// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/Actor.h"

#include "ActorTransactionAnnotation.h"
#include "Engine/LevelStreaming.h"
#include "EngineStats.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Engine/NetConnection.h"
#include "GameFramework/DamageType.h"
#include "GameFramework/WorldSettings.h"
#include "Net/Core/Misc/NetContext.h"
#include "Net/Core/PropertyConditions/PropertyConditions.h"
#include "TimerManager.h"
#include "GameFramework/Pawn.h"
#include "AI/NavigationSystemBase.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Engine.h"
#include "Net/Core/PropertyConditions/RepChangedPropertyTracker.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/CoreNetContext.h"
#include "UObject/ObjectSaveContext.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/PhysicsReplicationQuantization.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/Package.h"
#include "UObject/UE5SpecialProjectStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Level.h"
#include "Engine/LocalPlayer.h"
#include "Engine/InputDelegateBinding.h"
#include "Engine/LevelStreamingPersistent.h"
#include "Logging/MessageLog.h"
#include "Net/UnrealNetwork.h"
#include "Engine/Canvas.h"
#include "DisplayDebugHelpers.h"
#include "Engine/DemoNetDriver.h"
#include "Components/PawnNoiseEmitterComponent.h"
#include "Camera/CameraComponent.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "ObjectTrace.h"
#include "Engine/AutoDestroySubsystem.h"
#include "WorldPartition/WorldPartitionRuntimeCellInterface.h"
#include "AutoRTFM.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Conditionals/ReplicationCondition.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "LevelUtils.h"
#include "Components/DecalComponent.h"
#include "Components/LightComponent.h"
#include "GameFramework/InputSettings.h"
#include "Algo/AnyOf.h"
#include "PrimitiveSceneProxy.h"
#include "UObject/MetaData.h"
#include "Engine/DamageEvents.h"
#include "Engine/SCS_Node.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Engine/PackageMapClient.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteExecutor.h"
#include "UObject/RemoteObject.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/UObjectMigrationContext.h"
#endif

#if WITH_EDITOR
#include "Features/IModularFeatures.h"
#include "FoliageHelper.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#endif

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "WorldPartition/ActorInstanceGuids.h"
#include "Engine/CoreSettings.h"

DEFINE_LOG_CATEGORY(LogActor);

DECLARE_CYCLE_STAT(TEXT("PostActorConstruction"), STAT_PostActorConstruction, STATGROUP_Engine);

CSV_DEFINE_CATEGORY(ActorMigration, UE_WITH_REMOTE_OBJECT_HANDLE);

#if UE_BUILD_SHIPPING
#define DEBUG_CALLSPACE(Format, ...)
#else
#define DEBUG_CALLSPACE(Format, ...) UE_LOG(LogNet, VeryVerbose, Format, __VA_ARGS__);
#endif

#define LOCTEXT_NAMESPACE "Actor"

FMakeNoiseDelegate AActor::MakeNoiseDelegate = FMakeNoiseDelegate::CreateStatic(&AActor::MakeNoiseImpl);

/** Controls if a replicated component should call FReplicationSystemUtil::StopReplicatingActorComponent immediately when updating its replication status. */
static bool GbCallStopReplicationForActorComponentWhenUpdatingReplicatedComponent = false;
static FAutoConsoleVariableRef CVarStopReplicationForActorComponentWhenUpdatingReplicatedComponent(
	TEXT("net.Iris.CallStopReplicationForActorComponentWhenUpdatingReplicatedComponent"),
	GbCallStopReplicationForActorComponentWhenUpdatingReplicatedComponent,
	TEXT("If true, replicated ActorComponentrs will call stop replication if they are no longer replicated after a call to UpdateReplicatedComponent.."));

/** Selects if actor and actorcomponents will replicate subobjects using the registration list by default. */
extern bool GDefaultUseSubObjectReplicationList;

/** Enables optimizations to actor and component registration */
extern int32 GOptimizeActorRegistration;

#if !UE_BUILD_SHIPPING
FOnProcessEvent AActor::ProcessEventDelegate;
#endif

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)

/** Count of total actors created */
TSAN_ATOMIC(int32) CSVActorTotalCount = 0;
/** Map of Actor class names to count */
TMap<FName, int32> CSVActorClassNameToCountMap;
/** Critical section to control access to map */
FCriticalSection CSVActorClassNameToCountMapLock;

#endif // (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)

/** Detailed CSV Recording of Actor Net Stats */
#ifndef CSV_RECORD_DETAILED_ACTOR_NET_STATS
#define CSV_RECORD_DETAILED_ACTOR_NET_STATS (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING && UE_SERVER)
#endif // CSV_RECORD_DETAILED_ACTOR_NET_STATS

#if CSV_RECORD_DETAILED_ACTOR_NET_STATS
CSV_DEFINE_CATEGORY(Actor_FlushNetDormancy, false);
CSV_DEFINE_CATEGORY(Actor_ForceNetUpdate, false);
#endif // CSV_RECORD_DETAILED_ACTOR_NET_STATS

#if WITH_EDITOR
namespace ActorUtils
{
	bool CanDeleteOrReplaceCommon(const AActor* InActor, FText& OutReason)
	{
		if (!InActor->IsUserManaged())
		{
			OutReason = LOCTEXT("CanDeleteOrReplace_Error_UserManaged", "Actor is not user managed");
			return false;
		}

		// Actors in LevelInstances can't be deleted unless they are in a regular edit level instance (not property override)
		if (InActor->IsInLevelInstance() && !InActor->IsInEditLevelInstance())
		{
			OutReason = LOCTEXT("CanDeleteOrReplace_Error_NonEditLevelInstance", "Actor is in a non edit level instance");
			return false;
		}

		return true;
	}
}

#endif

uint32 AActor::BeginPlayCallDepth = 0;

namespace ActorUtils
{
	namespace CVars
	{
#if CSV_RECORD_DETAILED_ACTOR_NET_STATS
		// Which type of class name to record when the CSV categories: Actor_FlushNetDormancy or Actor_ForceNetUpdate are enabled
		// From least verbose to most verbose:
		// 	- 0: Record the Parent native class name of the given Actor
		// 	- 1: Record the TopMost non-native class name of the given Actor
		// 	- 2: Record the Actor class name
		int32 CsvActorClassVerbosity = 0;
		FAutoConsoleVariableRef CVarCsvActorClassVerbosity(
			TEXT("net.ForceFlush.CsvActorClassVerbosity")
			, CsvActorClassVerbosity
			, TEXT("Which Actor type of class name to record when the CSV categories: Actor_FlushNetDormancy or Actor_ForceNetUpdate are enabled"
					" 0: Record the Parent native class name of the given Actor."
					" 1: Record the TopMost non-native class name of the given Actor."
					" 2: Record the Actor class name.")
			, ECVF_Cheat
		);
#endif // CSV_RECORD_DETAILED_ACTOR_NET_STATS
	}

#if CSV_RECORD_DETAILED_ACTOR_NET_STATS
	static void CsvRecordClassName(UClass* Class, int32 CsvCategoryIndex)
	{
		FCsvProfiler* CsvProfiler = FCsvProfiler::Get();
		if (!CsvProfiler)
		{
			return;
		}

		if (!CsvProfiler->IsCapturing())
		{
			return;
		}

		if (!CsvProfiler->IsCategoryEnabled(CsvCategoryIndex))
		{
			return;
		}

		switch (ActorUtils::CVars::CsvActorClassVerbosity)
		{
			case 0: // Record the Parent native class name of the given Actor
			{
				Class = GetParentNativeClass(Class);
				break;
			}
			case 1: // Record the TopMost non-native class name of the given Actor
			{
				constexpr bool bAllowReturnNativeClass = true;
				Class = GetTopMostNonNativeClass(Class, bAllowReturnNativeClass);
				break;
			}
			case 2: // Record the Actor class name (reuse Class variable)
			{
				break;
			}
		}
				
		if (ensure(Class))
		{
			CsvProfiler->RecordCustomStat(*GetNameSafe(Class), CsvCategoryIndex, 1, ECsvCustomStatOp::Accumulate);
		}
	}
#endif // CSV_RECORD_DETAILED_ACTOR_NET_STATS


	template<class Function>
	static void ForEachNetDriver(UEngine* Engine, const UWorld* const World, const Function InFunction)
	{
		if (Engine == nullptr || World == nullptr)
		{
			return;
		}

		FWorldContext* const Context = Engine->GetWorldContextFromWorld(World);
		if (Context != nullptr)
		{
			for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
			{
				InFunction(Driver.NetDriver);
			}
		}
	}
}

AActor::AActor()
{
	InitializeDefaults();
}

AActor::AActor(const FObjectInitializer& ObjectInitializer)
{
	// Forward to default constructor (we don't use ObjectInitializer for anything, this is for compatibility with inherited classes that call Super( ObjectInitializer )
	InitializeDefaults();
}

void AActor::InitializeDefaults()
{
	PrimaryActorTick.TickGroup = TG_PrePhysics;
	// Default to no tick function, but if we set 'never ticks' to false (so there is a tick function) it is enabled by default
	PrimaryActorTick.bCanEverTick = false;
	PrimaryActorTick.bStartWithTickEnabled = true;
	PrimaryActorTick.bAllowTickBatching = true;
	PrimaryActorTick.bRunTransactionally = true;
	PrimaryActorTick.SetTickFunctionEnable(false); 
	bAsyncPhysicsTickEnabled = false;

	CustomTimeDilation = 1.0f;

	SetRole(ROLE_Authority);
	RemoteRole = ROLE_None;
	bReplicates = false;
	bCallPreReplication = true;
	bCallPreReplicationForReplay = true;
	bReplicateUsingRegisteredSubObjectList = GDefaultUseSubObjectReplicationList;
	PhysicsReplicationMode = EPhysicsReplicationMode::Default;
	NetPriority = 1.0f;
	SetNetUpdateFrequency(100.0f);
	SetMinNetUpdateFrequency(2.0f);
	bNetLoadOnClient = true;
#if WITH_EDITORONLY_DATA
	bEditable = true;
	bListedInSceneOutliner = true;
	bDefaultOutlinerExpansionState = true;
	bIsEditorPreviewActor = false;
	bHiddenEdLayer = false;
	bHiddenEdTemporary = false;
	bHiddenEdLevel = false;
	bActorLabelEditable = true;
	SpriteScale = 1.0f;
	bOptimizeBPComponentData = false;
	bForceExternalActorLevelReferenceForPIE = false;
#endif // WITH_EDITORONLY_DATA
	bEnableAutoLODGeneration = true;
	SetNetCullDistanceSquared(225000000.0f);
	NetDriverName = NAME_GameNetDriver;
	NetDormancy = DORM_Awake;
	// will be updated in PostInitProperties
	bActorEnableCollision = true;
	bActorSeamlessTraveled = false;
	bBlockInput = false;
	SetCanBeDamaged(true);
	bFindCameraComponentWhenViewTarget = true;
	bAllowReceiveTickEventOnDedicatedServer = true;
	bRelevantForNetworkReplays = true;
	bRelevantForLevelBounds = true;
	RayTracingGroupId = FPrimitiveSceneProxy::InvalidRayTracingGroupId;

	// Overlap collision settings
	bGenerateOverlapEventsDuringLevelStreaming = false;
	UpdateOverlapsMethodDuringLevelStreaming = EActorUpdateOverlapsMethod::UseConfigDefault;
	DefaultUpdateOverlapsMethodDuringLevelStreaming = EActorUpdateOverlapsMethod::OnlyUpdateMovable;
	
	bHasDeferredComponentRegistration = false;
	bHasPreRegisteredAllComponents = false;
	bHasRegisteredAllComponents = false;

	bActorIsPendingPostNetInit = false;

#if WITH_EDITORONLY_DATA
	LevelInstanceFlags = ELevelInstanceFlags::None;
	LevelInstanceType = ELevelInstanceType::None;
	PivotOffset = FVector::ZeroVector;
#endif
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
	// Increment actor class count
	// Update our ActorClassName count after the transaction finishes
	UE_AUTORTFM_ONCOMMIT(this)
	{
		if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
		{
			FScopeLock Lock(&CSVActorClassNameToCountMapLock);

			const UClass* ParentNativeClass = GetParentNativeClass(GetClass());
			FName NativeClassName = ParentNativeClass ? ParentNativeClass->GetFName() : NAME_None;
			int32& CurrentCount = CSVActorClassNameToCountMap.FindOrAdd(NativeClassName);
			CurrentCount++;
			CSVActorTotalCount++;
		}
	};
#endif // (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = true;
	CopyPasteId = INDEX_NONE;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GridPlacement_DEPRECATED = EActorGridPlacement::None;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

void FActorTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	if (IsValid(Target))
	{
		if (TickType != LEVELTICK_ViewportsOnly || Target->ShouldTickIfViewportsOnly())
		{
			FScopeCycleCounterUObject ActorScope(Target);
			Target->TickActor(DeltaTime*Target->CustomTimeDilation, TickType, *this);
		}

#if UE_WITH_REMOTE_OBJECT_HANDLE
		UE_AUTORTFM_OPEN
		{
			CachedDiagnosticContext = DiagnosticContext(true);
			CachedDiagnosticMessage = DiagnosticMessage();
		};
#endif
	}
}

FString FActorTickFunction::DiagnosticMessage()
{
	return Target->GetFullName() + TEXT("[TickActor]");
}

FName FActorTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		// Format is "ActorNativeClass/ActorClass"
		FString ContextString = FString::Printf(TEXT("%s/%s"), *GetParentNativeClass(Target->GetClass())->GetName(), *Target->GetClass()->GetName());
		return FName(*ContextString);
	}
	else
	{
		return GetParentNativeClass(Target->GetClass())->GetFName();
	}
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
void FActorTickFunction::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Target);
}
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
void FActorComponentTickFunction::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Target);
}
#endif

bool AActor::CheckDefaultSubobjectsInternal() const
{
	bool Result = Super::CheckDefaultSubobjectsInternal();
	if (Result)
	{
		Result = CheckActorComponents();
	}
	return Result;
}

bool AActor::CheckActorComponents() const
{
	DEFINE_LOG_CATEGORY_STATIC(LogCheckComponents, Warning, All);

	bool bResult = true;

	for (UActorComponent* Inner : GetComponents())
	{
		if (!Inner)
		{
			continue;
		}
		if (!IsValidChecked(Inner))
		{
			UE_LOGF(LogCheckComponents, Warning, "Component is invalid. Me = %ls, Component = %ls", *this->GetFullName(), *Inner->GetFullName());
		}
		if (Inner->IsTemplate() && !IsTemplate())
		{
			UE_LOGF(LogCheckComponents, Error, "Component is a template but I am not. Me = %ls, Component = %ls", *this->GetFullName(), *Inner->GetFullName());
			bResult = false;
		}
	}
	for (int32 Index = 0; Index < BlueprintCreatedComponents.Num(); Index++)
	{
		UActorComponent* Inner = BlueprintCreatedComponents[Index];
		if (!Inner)
		{
			continue;
		}
		if (Inner->GetOuter() != this)
		{
			UE_LOGF(LogCheckComponents, Error, "SerializedComponent does not have me as an outer. Me = %ls, Component = %ls", *this->GetFullName(), *Inner->GetFullName());
			bResult = false;
		}
		if (!IsValidChecked(Inner))
		{
			UE_LOGF(LogCheckComponents, Warning, "SerializedComponent is invalid. Me = %ls, Component = %ls", *this->GetFullName(), *Inner->GetFullName());
		}
		if (Inner->IsTemplate() && !IsTemplate())
		{
			UE_LOGF(LogCheckComponents, Error, "SerializedComponent is a template but I am not. Me = %ls, Component = %ls", *this->GetFullName(), *Inner->GetFullName());
			bResult = false;
		}
	}
	return bResult;
}

void AActor::ResetOwnedComponents()
{
	ResetOwnedComponents(EAddComponentForReplicationBehavior::Immediate);
}

void AActor::ResetOwnedComponents(EAddComponentForReplicationBehavior Type)
{
#if WITH_EDITOR
	// Identify any natively-constructed components referenced by properties that either failed to serialize or came in as nullptr.
	if(HasAnyFlags(RF_WasLoaded) && NativeConstructedComponentToPropertyMap.Num() > 0)
	{
		for (UActorComponent* Component : OwnedComponents)
		{
			// Only consider native components
			if (Component && Component->CreationMethod == EComponentCreationMethod::Native)
			{
				// Find the property or properties that previously referenced the natively-constructed component.
				TArray<FObjectProperty*> Properties;
				NativeConstructedComponentToPropertyMap.MultiFind(Component->GetFName(), Properties);

				// Determine if the property or properties are no longer valid references (either it got serialized out that way or something failed during load)
				for (FObjectProperty* ObjProp : Properties)
				{
					check(ObjProp != nullptr);
					UActorComponent* ActorComponent = Cast<UActorComponent>(ObjProp->GetObjectPropertyValue_InContainer(this));
					if (ActorComponent == nullptr)
					{
						// Restore the natively-constructed component instance
						ObjProp->SetObjectPropertyValue_InContainer(this, Component);
					}
				}
			}
		}

		// Clear out the mapping as we don't need it anymore
		NativeConstructedComponentToPropertyMap.Empty();
	}
#endif

	OwnedComponents.Reset();
	ReplicatedComponents.Reset();
	ReplicatedComponentsInfo.Reset();
	
	ForEachObjectWithOuter(this, [this, Type](UObject* Child)
	{
		UActorComponent* Component = Cast<UActorComponent>(Child);
		if (Component && Component->GetOwner() == this)
		{
			OwnedComponents.Add(Component);

			if (Component->GetIsReplicated())
			{
				ReplicatedComponents.Add(Component);

				AddComponentForReplication(Component, Type);
			}
		}
	}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);
}

void AActor::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	// This will be executed in PostLoad instead during loading to avoid registering objects too soon
	// before they are ready.
	if (!IsInAsyncLoadingThread())
	{
		UEngineElementsLibrary::CreateEditorActorElement(this);
	}
#endif	// WITH_EDITOR

	RemoteRole = (bReplicates ? ROLE_SimulatedProxy : ROLE_None);

	// Make sure the OwnedComponents list correct.  
	// Under some circumstances sub-object instancing can result in bogus/duplicate entries.
	// This is not necessary for CDOs
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ResetOwnedComponents();
	}
}

bool AActor::CanBeInCluster() const
{
	return bCanBeInCluster;
}

void AActor::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	AActor* This = CastChecked<AActor>(InThis);
	Collector.AddStableReferenceSet(&This->OwnedComponents);
#if WITH_EDITOR
	if (This->CurrentTransactionAnnotation.IsValid())
	{
		This->CurrentTransactionAnnotation->AddReferencedObjects(Collector);
	}
#endif
	Super::AddReferencedObjects(InThis, Collector);
}

bool AActor::IsEditorOnly() const
{ 
	return bIsEditorOnlyActor; 
}

bool AActor::IsAsset() const
{
	// External actors are considered assets, to allow using the asset logic for save dialogs, etc.
	// Also, they return true even if pending kill, in order to show up as deleted in these dialogs.
	return IsPackageExternal() && !GetPackage()->HasAnyFlags(RF_Transient) && !HasAnyFlags(RF_Transient | RF_ClassDefaultObject) && IsMainPackageActor() && !GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor);
}

void AActor::PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext)
{
	Super::PreSaveRoot(ObjectSaveContext);
#if WITH_EDITOR
	// if `PreSaveRoot` is called on an actor, it should be have its package overridden (external to the level)
	// if this actor is not in the current persistent level then we might need to remove level streamin transform before saving it
	ULevel* Level = GetLevel();
	if (IsPackageExternal() && Level && !Level->IsPersistentLevel())
	{
		// If we can get the streaming level, we should remove the editor transform before saving
		ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(Level);
		if (LevelStreaming && Level->bAlreadyMovedActors)
		{
			FLevelUtils::RemoveEditorTransform(LevelStreaming, true, this);
		}
		ObjectSaveContext.SetCleanupRequired(true);
	}
#endif
}

void AActor::PostSaveRoot(FObjectPostSaveRootContext  ObjectSaveContext)
{
	Super::PostSaveRoot(ObjectSaveContext);
#if WITH_EDITOR
	// if this actor is not in the current persistent level then we will need to clean up the removal of level streaming
	ULevel* Level = GetLevel();
	if (ObjectSaveContext.IsCleanupRequired() && IsPackageExternal() && Level && !Level->IsPersistentLevel())
	{
		// If we can get the streaming level, we should remove the editor transform before saving
		ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(Level);
		if (LevelStreaming && Level->bAlreadyMovedActors)
		{
			FLevelUtils::ApplyEditorTransform(LevelStreaming, true, this);
		}
	}
#endif
}

void AActor::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
#if WITH_EDITOR
	// Always call FixupDataLayers() on PreSave so that DataLayerAssets SoftObjectPtrs internal WeakPtrs get resolved before UE::IsSavingPackage() gets set to true preventing resolving.
	FixupDataLayers();
#endif
}

#if WITH_EDITOR
void AActor::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (IsPackageExternal() && !IsChildActor())
	{
		if (ActorLabel.Len() > 0)
		{
			static FName NAME_ActorLabel("ActorLabel");
			Context.AddTag(FAssetRegistryTag(NAME_ActorLabel, ActorLabel, UObject::FAssetRegistryTag::TT_Hidden));
		}
	}

	if (Context.IsSaving())
	{
		if (IsPackageExternal() && !IsChildActor())
		{
			FWorldPartitionActorDescUtils::AppendAssetDataTagsFromActor(this, Context);
		}
	}
}

bool AActor::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const
{
	// this is expected to be by far the most common case, saves us some time in the cook.
	if (!RootComponent || RootComponent->DetailMode == EDetailMode::DM_Low)
	{
		return true;
	}

	if(UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(TargetPlatform->IniPlatformName()))
	{
		// get local scalability CVars that could cull this actor
		int32 CVarCullBasedOnDetailLevel;
		if(DeviceProfile->GetConsolidatedCVarValue(TEXT("r.CookOutUnusedDetailModeComponents"), CVarCullBasedOnDetailLevel) && CVarCullBasedOnDetailLevel == 1)
		{
			int32 CVarDetailMode;
			if(DeviceProfile->GetConsolidatedCVarValue(TEXT("r.DetailMode"), CVarDetailMode))
			{
				// Check root component's detail mode.
				// If e.g. the component's detail mode is High and the platform detail is Medium,
				// then we should cull it.
				if((int32)RootComponent->DetailMode > CVarDetailMode)
				{
					return false;
				}
			}
		}
	}

	return true;
}

void AActor::SetBrowseToAssetOverride(const FString& PackageName)
{
	GetPackage()->GetMetaData().SetValue(this, "BrowseToAssetOverride", *PackageName);
}

const FString& AActor::GetBrowseToAssetOverride() const
{
	return GetPackage()->GetMetaData().GetValue(this, "BrowseToAssetOverride");
}

#endif

UWorld* AActor::GetWorld() const
{
	// CDO objects do not belong to a world
	// If the actors outer is destroyed or unreachable we are shutting down and the world should be nullptr
	if (!HasAnyFlags(RF_ClassDefaultObject) && ensureMsgf(GetOuter(), TEXT("Actor: %s has a null OuterPrivate in AActor::GetWorld()"), *GetFullName())
		&& !GetOuter()->HasAnyFlags(RF_BeginDestroyed) && !GetOuter()->IsUnreachable())
	{
		if (ULevel* Level = GetLevel())
		{
			return Level->OwningWorld;
		}
	}
	return nullptr;
}

FTimerManager& AActor::GetWorldTimerManager() const
{
	return GetWorld()->GetTimerManager();
}

UGameInstance* AActor::GetGameInstance() const
{
	if (const UWorld* World = GetWorld())
	{
		return World->GetGameInstance();
	}

	return nullptr;
}

bool AActor::IsNetStartupActor() const
{
	// Check bNetStartup and also check if this is a Net Startup Actor that has not been initialized and has not had a chance to flag bNetStartup yet
	return bNetStartup || (!bActorInitialized && !bActorSeamlessTraveled && bNetLoadOnClient && GetLevel() && !GetLevel()->bAlreadyInitializedNetworkActors);
}

FVector AActor::GetVelocity() const
{
	if ( RootComponent )
	{
		return RootComponent->GetComponentVelocity();
	}

	return FVector::ZeroVector;
}

void AActor::ClearCrossLevelReferences()
{
	if(RootComponent && GetRootComponent()->GetAttachParent() && (GetLevel() != GetRootComponent()->GetAttachParent()->GetOwner()->GetLevel()))
	{
		GetRootComponent()->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
	}
}

bool AActor::TeleportTo( const FVector& DestLocation, const FRotator& DestRotation, bool bIsATest, bool bNoCheck )
{
	SCOPE_CYCLE_COUNTER(STAT_TeleportToTime);

	if(RootComponent == nullptr)
	{
		return false;
	}

	UWorld* MyWorld = GetWorld();

	// Can't move non-movable actors during play
	if( (RootComponent->Mobility == EComponentMobility::Static) && MyWorld->AreActorsInitialized() )
	{
		return false;
	}

	if ( bIsATest && (GetActorLocation() == DestLocation) )
	{
		return true;
	}

	FVector const PrevLocation = GetActorLocation();
	FVector NewLocation = DestLocation;
	bool bTeleportSucceeded = true;
	UPrimitiveComponent* ActorPrimComp = Cast<UPrimitiveComponent>(RootComponent);
	if ( ActorPrimComp )
	{
		if (!bNoCheck && (ActorPrimComp->IsQueryCollisionEnabled() || (GetNetMode() != NM_Client)) )
		{
			// Apply the pivot offset to the desired location
			FVector Offset = GetRootComponent()->Bounds.Origin - PrevLocation;
			NewLocation = NewLocation + Offset;

			// check if able to find an acceptable destination for this actor that doesn't embed it in world geometry
			bTeleportSucceeded = MyWorld->FindTeleportSpot(this, NewLocation, DestRotation);
			NewLocation = NewLocation - Offset;
		}

		if (NewLocation.ContainsNaN() || PrevLocation.ContainsNaN())
		{
			bTeleportSucceeded = false;
			UE_LOGF(LogActor, Log,  "Attempted to teleport to NaN");
		}

		if ( bTeleportSucceeded )
		{
			// check whether this actor unacceptably encroaches on any other actors.
			if ( bIsATest || bNoCheck )
			{
				ActorPrimComp->SetWorldLocationAndRotation(NewLocation, DestRotation);
			}
			else
			{
				FVector const Delta = NewLocation - PrevLocation;
				bTeleportSucceeded = ActorPrimComp->MoveComponent(Delta, DestRotation, false, nullptr, MOVECOMP_NoFlags, ETeleportType::TeleportPhysics);
			}
			if( bTeleportSucceeded )
			{
				TeleportSucceeded(bIsATest);
			}
		}
	}
	else if (RootComponent)
	{
		// not a primitivecomponent, just set directly
		GetRootComponent()->SetWorldLocationAndRotation(NewLocation, DestRotation, false, nullptr, ETeleportType::TeleportPhysics);
	}

	return bTeleportSucceeded; 
}


bool AActor::K2_TeleportTo( FVector DestLocation, FRotator DestRotation )
{
	return TeleportTo(DestLocation, DestRotation, false, false);
}

void AActor::AddTickPrerequisiteActor(AActor* PrerequisiteActor)
{
	if (PrimaryActorTick.bCanEverTick && PrerequisiteActor && PrerequisiteActor->PrimaryActorTick.bCanEverTick)
	{
		PrimaryActorTick.AddPrerequisite(PrerequisiteActor, PrerequisiteActor->PrimaryActorTick);
	}
}

void AActor::AddTickPrerequisiteComponent(UActorComponent* PrerequisiteComponent)
{
	if (PrimaryActorTick.bCanEverTick && PrerequisiteComponent && PrerequisiteComponent->PrimaryComponentTick.bCanEverTick)
	{
		PrimaryActorTick.AddPrerequisite(PrerequisiteComponent, PrerequisiteComponent->PrimaryComponentTick);
	}
}

void AActor::RemoveTickPrerequisiteActor(AActor* PrerequisiteActor)
{
	if (PrerequisiteActor)
	{
		PrimaryActorTick.RemovePrerequisite(PrerequisiteActor, PrerequisiteActor->PrimaryActorTick);
	}
}

void AActor::RemoveTickPrerequisiteComponent(UActorComponent* PrerequisiteComponent)
{
	if (PrerequisiteComponent)
	{
		PrimaryActorTick.RemovePrerequisite(PrerequisiteComponent, PrerequisiteComponent->PrimaryComponentTick);
	}
}

bool AActor::GetTickableWhenPaused()
{
	return PrimaryActorTick.bTickEvenWhenPaused;
}

void AActor::SetTickableWhenPaused(bool bTickableWhenPaused)
{
	PrimaryActorTick.bTickEvenWhenPaused = bTickableWhenPaused;
}

void AActor::BeginDestroy()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	// Remote actors have already gone through the pcoesses of removing from world and by repeating it we'd migrate them back
	if (!HasAnyInternalFlags(EInternalObjectFlags::Remote))
	{
		UnregisterAllComponents();
		ULevel* Level = GetLevel();
		if (Level && !Level->IsUnreachable())
		{
			Level->Actors.RemoveSingleSwap(this, EAllowShrinking::No);
		}
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
	// Decrement actor class count
	// Update our ActorClassName count after the transaction finishes
	UE_AUTORTFM_ONCOMMIT(this)
	{
		if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
		{
			FScopeLock Lock(&CSVActorClassNameToCountMapLock);

			const UClass* ParentNativeClass = GetParentNativeClass(GetClass());
			FName NativeClassName = ParentNativeClass ? ParentNativeClass->GetFName() : NAME_None;
			int32* CurrentCount = CSVActorClassNameToCountMap.Find(NativeClassName);
			if (CurrentCount)
			{
				(*CurrentCount)--;
			}
			CSVActorTotalCount--;
		}
	};
#endif // (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)

#if WITH_EDITOR
	UEngineElementsLibrary::DestroyEditorActorElement(this);
#endif	// WITH_EDITOR

	Super::BeginDestroy();
}

void AActor::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5SpecialProjectStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITOR
	// Prior to load, map natively-constructed component instances for Blueprint-generated class types to any serialized properties that might reference them.
	// We'll use this information post-load to determine if any owned components may not have been serialized through the reference property (i.e. in case the serialized property value ends up being NULL).
	if (Ar.IsLoading()
		&& OwnedComponents.Num() > 0
		&& !(Ar.GetPortFlags() & PPF_Duplicate)
		&& HasAllFlags(RF_WasLoaded|RF_NeedPostLoad))
	{
		if (const UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(GetClass()))
		{
			NativeConstructedComponentToPropertyMap.Reset();
			NativeConstructedComponentToPropertyMap.Reserve(OwnedComponents.Num());
			for(TFieldIterator<FObjectProperty> PropertyIt(BPGC, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				FObjectProperty* ObjProp = *PropertyIt;

				// Ignore transient properties since they won't be serialized
				if(!ObjProp->HasAnyPropertyFlags(CPF_Transient))
				{
					const TObjectPtr<UObject>& ObjectPtr = ObjProp->GetPropertyValue_InContainer(this);
					UActorComponent* ActorComponent = Cast<UActorComponent>(ObjectPtr);
					if(ActorComponent != nullptr && ActorComponent->CreationMethod == EComponentCreationMethod::Native)
					{
						NativeConstructedComponentToPropertyMap.Add(ActorComponent->GetFName(), ObjProp);
					}
				}
			}
		}
	}

	// When duplicating for PIE all components need to be gathered up and duplicated even if there are no other property references to them
	// otherwise we can end up with Attach Parents that do not get redirected to the correct component. However, if there is a transient component
	// we'll let that drop
	if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
	{
		TInlineComponentArray<UActorComponent*> DuplicatingComponents;
		if (Ar.IsSaving())
		{
			DuplicatingComponents.Reserve(OwnedComponents.Num());
			for (UActorComponent* OwnedComponent : OwnedComponents)
			{
				if (OwnedComponent && !OwnedComponent->HasAnyFlags(RF_Transient))
				{
					DuplicatingComponents.Add(OwnedComponent);
				}
			}
		}
		Ar << DuplicatingComponents;
	}
#endif

	Super::Serialize(Ar);

	// Always serialize the actor label in cooked builds
	if(Ar.IsPersistent() && (Ar.CustomVer(FUE5SpecialProjectStreamObjectVersion::GUID) >= FUE5SpecialProjectStreamObjectVersion::SerializeActorLabelInCookedBuilds))
	{
		bool bIsCooked = Ar.IsCooking();
		Ar << bIsCooked;

		if (bIsCooked)
		{
#if !(WITH_EDITORONLY_DATA || ACTOR_HAS_LABELS)
			// In non-development builds, just skip over the actor labels. We need to figure out a way to either strip that data from shipping builds,
			// or skip over the string data without doing any memory allocations, probably with a custom FString::SerializeToNull implementation.
			FString ActorLabel;
#endif
			Ar << ActorLabel;
		}
	}

#if WITH_EDITOR
	// Fixup actor guids
	if (Ar.IsLoading())
	{
		if (IsTemplate())
		{
			if (ActorGuid.IsValid())
			{
				ActorGuid.Invalidate();
			}
		}
		else if (Ar.IsPersistent() && !ActorGuid.IsValid())
		{
			ActorGuid = FGuid::NewDeterministicGuid(GetPathName());
		}
		else if ((Ar.GetPortFlags() & (PPF_Duplicate | PPF_DuplicateForPIE)) == PPF_Duplicate)
		{
			ActorGuid = FGuid::NewGuid();
		}

		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::ActorGridPlacementDeprecateDefaultValueFixup)
		{
			if (!CanChangeIsSpatiallyLoadedFlag())
			{
				bIsSpatiallyLoaded = GetClass()->GetDefaultObject<AActor>()->bIsSpatiallyLoaded;
			}
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeContentBundleGuid)
		{
			ContentBundleGuid = ContentBundlePaths::GetContentBundleGuidFromExternalActorPackagePath(GetPackage()->GetFName().ToString());
		}
		else // @todo_ow: remove once we find why some actors end up with invalid ContentBundleGuids
		{
			FGuid FixupContentBundleGuid = ContentBundlePaths::GetContentBundleGuidFromExternalActorPackagePath(GetPackage()->GetFName().ToString());
			if (ContentBundleGuid != FixupContentBundleGuid)
			{
				UE_LOGF(LogActor, Log, "Actor ContentBundleGuid was fixed up: %ls", *GetName());
				ContentBundleGuid = FixupContentBundleGuid;
			}
		}
	}
#endif

	if ((Ar.IsCooking() || Ar.IsLoadingFromCookedPackage()) && (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::LevelInstanceStaticLightingSupport))
	{		
		FActorInstanceGuid::Serialize(Ar, *this);
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Ar.IsMigratingRemoteObjects())
	{
		uint8 MigrationFlags = 0;

		// TODO: This a workaround for not currently supporting native delegates in object migration
		// we are treating LifeSpan as a property of Actor rather than solving TimerManager's delegates generically.
		// Calling SetLifeSpan here is also not ideal because it's virtual which means it can do "anything" while in Serialize.
		// We could fix it with FUObjectAnnotationSparse
		float CurrentLifespan = 0.0f; 
		if (Ar.IsSaving())
		{
			CurrentLifespan = GetLifeSpan();
			SetLifeSpan(0.0f);

			MigrationFlags |= (static_cast<uint8>(ActorHasBegunPlay) & 3) << 0;
			MigrationFlags |= (bHasFinishedSpawning & 1) << 2;
			MigrationFlags |= (bActorInitialized & 1) << 3;
			MigrationFlags |= (bTickFunctionsRegistered & 1) << 4;
		}

		Ar << CreationTime;
		Ar << CurrentLifespan;
		Ar << MigrationFlags;
		Ar << OwnedComponents;

		UE::RemoteObject::Serialization::Network::SerializeRPCQueue(this, Ar);

		if (Ar.IsLoading())
		{
			// When migrating actors are in a state after BeginPlay has been called but we need to call SetLifeSpan to initialize local timer
			SetLifeSpan(CurrentLifespan);

			ActorHasBegunPlay = static_cast<EActorBeginPlayState>(MigrationFlags & 3);
			bHasFinishedSpawning = (MigrationFlags >> 2) & 0x1;
			bActorInitialized = (MigrationFlags >> 3) & 0x1;
			bTickFunctionsRegistered = (MigrationFlags >> 4) & 0x1;
		}
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
}

void AActor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	UEngineElementsLibrary::CreateEditorActorElement(this);
#endif

	// add ourselves to our Owner's Children array
	if (Owner != nullptr)
	{
		checkSlow(!Owner->Children.Contains(this));
		Owner->Children.Add(this);
	}

	if (GetLinkerUEVersion() < VER_UE4_PRIVATE_REMOTE_ROLE)
	{
		bReplicates = (RemoteRole != ROLE_None);
	}

	// Ensure that this is not set for CDO (there was a case where this might have occurred in an older version when converting actor instances to BPs - see UE-18490)
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		bExchangedRoles = false;
	}

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::ExternalActorsMapDataPackageFlag)
	{
		if (IsPackageExternal())
		{
			GetExternalPackage()->SetPackageFlags(PKG_ContainsMapData);
		}
	}
#endif

#if WITH_EDITORONLY_DATA
	if (AActor* ParentActor = ParentComponentActor_DEPRECATED.Get())
	{
		TInlineComponentArray<UChildActorComponent*> ParentChildActorComponents(ParentActor);
		for (UChildActorComponent* ChildActorComponent : ParentChildActorComponents)
		{
			if (ChildActorComponent->GetChildActor() == this)
			{
				ParentComponent = ChildActorComponent;
				break;
			}
		}
	}

	if ( GIsEditor )
	{
		// Propagate the hidden at editor startup flag to the transient hidden flag
		bHiddenEdTemporary = bHiddenEd;
	}

	// Fixup DataLayers
	FixupDataLayers();

	// Deprecate GridPlacement
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GridPlacement_DEPRECATED != EActorGridPlacement::None)
	{
		bIsSpatiallyLoaded = GridPlacement_DEPRECATED != EActorGridPlacement::AlwaysLoaded;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	// Since the actor is being loading, it finished spawning by definition when it was originally spawned, so set to true now
	bHasFinishedSpawning = true;
}

void AActor::PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph)
{
	USceneComponent* OldRoot = RootComponent;
	USceneComponent* OldRootParent = (OldRoot ? OldRoot->GetAttachParent() : nullptr);
	bool bHadRoot = !!OldRoot;
	FRotator OldRotation = FRotator::ZeroRotator;
	FVector OldTranslation = FVector::ZeroVector;
	FVector OldScale = FVector::ZeroVector;
	if (bHadRoot)
	{
		OldRotation = OldRoot->GetRelativeRotation();
		OldTranslation = OldRoot->GetRelativeLocation();
		OldScale = OldRoot->GetRelativeScale3D();
	}

	Super::PostLoadSubobjects(OuterInstanceGraph);

	ResetOwnedComponents();

	// If a constructor changed the component hierarchy of a serialized actor, it's possible to end up with a circular
	// attachment in the hierarchy. Fix that using the archetype as a guideline.
	if (GetAttachParentActor() == this)
	{
		// detected circular attachment. Fixup using archetype
		const USceneComponent* ArchetypeRoot = CastChecked<AActor>(GetArchetype())->GetRootComponent();
		TInlineComponentArray<USceneComponent*> Applicants;
		GetComponents(ArchetypeRoot->GetClass(), Applicants);
		for (USceneComponent* Applicant : Applicants)
		{
			if (Applicant->GetFName() == ArchetypeRoot->GetFName())
			{
				SetRootComponent(Applicant);
				break;
			}
		}
	}

	// Redirect the root component away from a Blueprint-added instance if the native ctor now constructs a default subobject as the root.
	if (RootComponent && !RootComponent->HasAnyFlags(RF_DefaultSubObject))
	{
		// Get the native class default object from which non-native classes (i.e. Blueprint classes) should inherit their root component.
		const AActor* ActorCDO = GetClass()->GetDefaultObject<AActor>();
		checkSlow(ActorCDO);

		// Ensure the root component references the appropriate instance; non-native classes (e.g. Blueprints) should always inherit from the CDO.
		if (const USceneComponent* ActorCDO_RootComponent = ActorCDO->GetRootComponent())
		{
			if (USceneComponent* ActorInstance_NativeRootComponent = Cast<USceneComponent>(GetDefaultSubobjectByName(ActorCDO_RootComponent->GetFName())))
			{
				SetRootComponent(ActorInstance_NativeRootComponent);
			}
		}
	}

	if (RootComponent && bHadRoot && OldRoot != RootComponent && OldRoot->IsIn(this))
	{
		UE_LOGF(LogActor, Log, "Root component has changed, relocating new root component to old position %ls->%ls", *OldRoot->GetFullName(), *GetRootComponent()->GetFullName());
		GetRootComponent()->SetRelativeRotation_Direct(OldRotation);
		GetRootComponent()->SetRelativeLocation_Direct(OldTranslation);
		GetRootComponent()->SetRelativeScale3D_Direct(OldScale);
		
		// Migrate any attachment to the new root
		if (OldRoot->GetAttachParent())
		{
			// Users may try to fixup attachment to the root on their own, avoid creating a cycle.
			if (OldRoot->GetAttachParent() != RootComponent)
			{
				RootComponent->SetupAttachment(OldRoot->GetAttachParent());
			}
		}

		// Attach old root to new root, if the user did not do something on their own during construction that differs from the serialized value.
		if (OldRoot->GetAttachParent() == OldRootParent && OldRoot->GetAttachParent() != RootComponent)
		{
			UE_LOGF(LogActor, Log, "--- Attaching old root to new root %ls->%ls", *OldRoot->GetFullName(), *GetRootComponent()->GetFullName());
			OldRoot->SetupAttachment(RootComponent);
		}

		// Reset the transform on the old component
		OldRoot->SetRelativeRotation_Direct(FRotator::ZeroRotator);
		OldRoot->SetRelativeLocation_Direct(FVector::ZeroVector);
		OldRoot->SetRelativeScale3D_Direct(FVector(1.0f, 1.0f, 1.0f));
	}
}

#if UE_WITH_REMOTE_OBJECT_HANDLE

// Initialize the static delegate
decltype(AActor::GetPreferredServerDelegate) AActor::GetPreferredServerDelegate;

static USceneComponent* FixupNativeActorComponents(AActor* Actor);

void AActor::PostMigrate(const FUObjectMigrationContext& MigrationContext)
{
	CSV_CUSTOM_STAT(ActorMigration, Total, 1, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(ActorMigration, Loan, MigrationContext.MigrationSide == EObjectMigrationSide::Send && MigrationContext.GetObjectMigrationSendType(this) == EObjectMigrationSendType::Loan, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(ActorMigration, ReturnBorrowed, MigrationContext.MigrationSide == EObjectMigrationSide::Send && MigrationContext.GetObjectMigrationSendType(this) == EObjectMigrationSendType::ReturnBorrowed, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(ActorMigration, ReassignBorrowed, MigrationContext.MigrationSide == EObjectMigrationSide::Send && MigrationContext.GetObjectMigrationSendType(this) == EObjectMigrationSendType::ReassignBorrowed, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(ActorMigration, ReassignOwnership, MigrationContext.MigrationSide == EObjectMigrationSide::Send && MigrationContext.GetObjectMigrationSendType(this) == EObjectMigrationSendType::ReassignOwnership, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(ActorMigration, Borrowed, MigrationContext.MigrationSide == EObjectMigrationSide::Receive && MigrationContext.GetObjectMigrationRecvType(this) == EObjectMigrationRecvType::Borrowed, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(ActorMigration, ReturnedLoan, MigrationContext.MigrationSide == EObjectMigrationSide::Receive && MigrationContext.GetObjectMigrationRecvType(this) == EObjectMigrationRecvType::ReturnedLoan, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(ActorMigration, AssignedOwnership, MigrationContext.MigrationSide == EObjectMigrationSide::Receive && MigrationContext.GetObjectMigrationRecvType(this) == EObjectMigrationRecvType::AssignedOwnership, ECsvCustomStatOp::Accumulate);

	// We want to be able to store "annotations" (metadata) about this Actor while we loan it out, so that
	// when we have the loan returned to us, we can check what has changed...
	struct FActorLoanedState
	{
		bool bActorWasDestroyed = false;
		bool bTickFunctionsRegistered = false;
		bool operator==(const FActorLoanedState&) const = default;
		bool IsDefault() const
		{
			static const FActorLoanedState DefaultLoanedState;
			return *this == DefaultLoanedState;
		}
	};
	static FUObjectAnnotationSparse<FActorLoanedState,/*bAutoRemove=*/true> RemoteObjects_ActorLoanedStates;

	Super::PostMigrate(MigrationContext);

	// We are using this as a proxy for having ownership until we sort out those mechanics
	const FRemoteServerId LocalServerId = FRemoteServerId::GetLocalServerId();
	const bool bHasOwnership = UE::RemoteObject::Handle::IsOwned(this);

	UWorld* const World = GetWorld();
	checkf(World, TEXT("Migrated Actor %s (%s) doesn't have a World"), *GetPathName(), *FRemoteObjectId(this).ToString());

	TGuardValue<ITransaction*> SuppressTransaction(GUndo, nullptr);
	if (MigrationContext.MigrationSide == EObjectMigrationSide::Send)
	{
		ensureMsgf(MigrationContext.RemoteServerId != LocalServerId, TEXT("%hs: Sending Object %s from %s to %s but destination is the same as the Local Server Id."), __func__, *GetName(), *LocalServerId.ToString(), *MigrationContext.RemoteServerId.ToString());

		// Remove the actor and it's sub-objects from the network id cache since these objects are now considered remote. It's assumed that the
		// network id cache will borrow any objects that are not in the cache but are considered remote.
		//
		// Loan is an exception because the server is still retaining ownership of this object and can still be dereferenced.
		const EObjectMigrationSendType MigrationSendType = MigrationContext.GetObjectMigrationSendType(this);
		if (MigrationSendType != EObjectMigrationSendType::Loan)
		{
			GetWorld()->GetNetDriver()->RemoveObjectFromNetworkIdCache(this);
		}

		// If we're loaning, we're keeping ownership; in all other cases we either don't own, or won't own the object after it's migrated
		const bool bKeepOwnership = (MigrationSendType == EObjectMigrationSendType::Loan);
		if (bKeepOwnership)
		{
			RemoteObjects_ActorLoanedStates.AddAnnotation(this, { .bActorWasDestroyed = (bool)bActorIsBeingDestroyed, .bTickFunctionsRegistered = (bool)bTickFunctionsRegistered });
		}
		else 
		{
			// Remove the actor from the world on this server as it is about to be migrated to another server.
			// Migrated actors are unique to the server instance that currently owns their memory and are not replicated to other servers.
			if (World)
			{
				UE::Net::Private::FNetPropertyConditionManager::Get().NotifyObjectDestroyed(this);

				if (FWorldContext* Context = GEngine->GetWorldContextFromWorld(World))
				{
					for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
					{
						if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(this))
						{
							Driver.NetDriver->NotifyActorDestroyed(this, false, EChannelCloseReason::Migrated);
						}
					}
				}

				// Remove the actor from the actor list.
				World->RemoveActor(this, /*bShouldModifyLevel*/ false);
			}

			// Disabling Ticks prevents them from happening *this frame* while unregistering them stops them from happening *next frame*
			PrimaryActorTick.SetTickFunctionEnable(false);
			ForEachComponent(true, [](UActorComponent* ActorComponent)
			{
				ActorComponent->PrimaryComponentTick.SetTickFunctionEnable(false);
			});

			// Clean up the actor's components.
			UnregisterAllComponents();

			// Unregister the actor's tick function
			RegisterAllActorTickFunctions(/*bRegisterTickFunctions*/ false, /*bIncludeComponents*/ true);
		}

		if (MigrationSendType == EObjectMigrationSendType::ReassignOwnership)
		{
			FStopReplicatingActorParams StopRepParams(EEndPlayReason::Destroyed);
			StopRepParams.StopReplicatingReason = UE::Net::EStopReplicatingReason::ProxyReuse;
			UE::Net::FDeferredReplicationSystemCalls::StopReplicatingActorDefer(this, StopRepParams);
		}
	}
	else if (MigrationContext.MigrationSide == EObjectMigrationSide::Receive)
	{
		// Build up the list of Components we have again (even if we previously owned this object, it could have been mutated)
		EAddComponentForReplicationBehavior AddComponentForRepType = EAddComponentForReplicationBehavior::NeverAdd;
		ResetOwnedComponents(AddComponentForRepType);

		// We can get rid of this check further down the road...
		// Set the actor's world transform if it has a native rootcomponent.
		const USceneComponent* SceneRootComponent = FixupNativeActorComponents(this);
		ensureMsgf(SceneRootComponent == RootComponent, TEXT("%hs: We were expecting RootComponent (%s) to be equal to FixupNativeActorComponents (%s). Transform fix-up code would be required to handle this case."), __func__, *GetNameSafe(RootComponent), *GetNameSafe(SceneRootComponent));

		// Add the actor that was migrated from another server to this server's world
		if (!bActorIsBeingDestroyed)
		{
			constexpr bool bOnlyAddIfUnique = true;
			GetLevel()->TryAddActorToList(this, bOnlyAddIfUnique);

			// Add the actor and it's sub-objects to the network id cache since these objects are no longer considered remote and can be derferenced
			// through the network id cache (e.g. by RPC argument and object deserialization).
			GetWorld()->GetNetDriver()->AddObjectToNetworkIdCache(this);
		}

		// If we're being passed ownership, register us with the global systems
		const EObjectMigrationRecvType MigrationRecvType = MigrationContext.GetObjectMigrationRecvType(this);
		if (MigrationRecvType == EObjectMigrationRecvType::AssignedOwnership)
		{
			ensureMsgf(!bActorIsBeingDestroyed, TEXT("We do not want to be assigned ownership of Actors that have already been destroyed. We do not yet know what to do with it."));

			// Any queued up RPCs will be processed when the deferred start replication is processed.
			UE::Net::FDeferredReplicationSystemCalls::StartReplicatingActorDefer(this);

			// Each component will be in charge of registering itself in its own PostMigrate, but we must register
			// the Actor (which we'll do be saying it's currently the opposite of the deserialized value, then flipping it)
			constexpr bool bRegisterComponents = false;
			bTickFunctionsRegistered = !bTickFunctionsRegistered;
			RegisterAllActorTickFunctions(!bTickFunctionsRegistered, bRegisterComponents);

			World->AddNetworkActor(this);
		}
		else if (MigrationRecvType == EObjectMigrationRecvType::ReturnedLoan)
		{
			// Only process queued RPCs if the actor is already registered with Iris (otherwise they will be ignored).
			// If not yet registered, the RPCs will be processed in FlushDeferred.
			if (UReplicationSystem* ReplicationSystem = GetNetDriver()->GetReplicationSystem())
			{
				if (const UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
				{
					const UE::Net::FNetRefHandle RefHandle = Bridge->GetReplicatedRefHandle(this);
					if (RefHandle.IsValid())
					{
						UE::RemoteObject::Serialization::Network::ProcessRPCQueue(this);
					}
				}
			}

			FActorLoanedState LoanedState = RemoteObjects_ActorLoanedStates.GetAndRemoveAnnotation(this);
			if (bActorIsBeingDestroyed)
			{
				// If we weren't destroyed when we loaned it out, we should destroy it now that we've received it back...
				if (!LoanedState.bActorWasDestroyed)
				{
					ensureMsgf(IsPendingKillPending(), TEXT("%hs: We thought %s was destroyed while on loan, but our received state does not match a destroyed Actor"), __func__, *GetName());
					UE_LOGF(LogActor, Warning, "%s: Destroying %ls back on owning server (it was destroyed while loaned-out)", __func__, *GetName());

					// Destroy it again on the owning server, as it needs to be deregistered from our systems
					bActorIsBeingDestroyed = false;
					ClearGarbage();
					World->DestroyActor(this);
				}
			}
			else if (bTickFunctionsRegistered != LoanedState.bTickFunctionsRegistered)
			{
				// Remote server changed the tick registration
				constexpr bool bRegisterComponents = false;
				bTickFunctionsRegistered = LoanedState.bTickFunctionsRegistered;
				RegisterAllActorTickFunctions(!bTickFunctionsRegistered, bRegisterComponents);
			}
		}
		else if (MigrationRecvType == EObjectMigrationRecvType::Borrowed)
		{
			// Make sure that all tick functions have been deregistered if we don't have ownership.
			bool bTickFunctionsRegisteredOnOwningServer = bTickFunctionsRegistered;
			RegisterAllActorTickFunctions(false, false);
			bTickFunctionsRegistered = bTickFunctionsRegisteredOnOwningServer;

			// We will always add ourselves to the list when borrowed, even if destroyed
			// The reason is that we never want to GC it, and always want to return it during a sync point
			if (bActorIsBeingDestroyed)
			{
				constexpr bool bOnlyAddIfUnique = true;
				GetLevel()->TryAddActorToList(this, bOnlyAddIfUnique);
			}
		}

		ActorHasBegunPlay = EActorBeginPlayState::HasBegunPlay;
	}
}

bool AActor::IsMigrationRoot() const
{
	return true;
}

FRemoteServerId AActor::GetPreferredRemoteServer() const
{
	if (GetPreferredServerDelegate.IsBound())
	{
		// Only use spatial lookup if the actor has a local, valid, movable root component.
		// Access RootComponent TObjectPtr without dereferencing — GetResidence() and
		// nullptr checks are non-dereferencing. Only call .Get() once confirmed Local.
		if (RootComponent.GetResidence() == EResidence::Local)
		{
			// Using a BP-generated default scene root.  This happens when a C++ class in inherited in Blueprint
			// and no other components are added.  So we want to treat these actors as having no root component
			// because they effectively have no transform thus should not be migrated.
			const bool bBlueprintHasRootComponent = (RootComponent.GetFName() != USceneComponent::GetDefaultSceneRootVariableName());

			USceneComponent* Root = RootComponent.Get();
			if (Root && Root->IsRegistered() && Root->Mobility == EComponentMobility::Movable
				&& bBlueprintHasRootComponent)
			{
				return GetPreferredServerDelegate.Execute(this);
			}
		}
	}
	return UE::RemoteObject::Handle::GetOwnerServerId(this);
}

#endif // UE_WITH_REMOTE_OBJECT_HANDLE

void AActor::ProcessEvent(UFunction* Function, void* Parameters)
{
	LLM_SCOPE(ELLMTag::EngineMisc);

#if !UE_BUILD_SHIPPING
	if (!ProcessEventDelegate.IsBound())
#endif
	{
		// Apply UObject::ProcessEvent's early outs before doing any other work
		// If the process event delegate is bound, we need to allow the process to play out
		if ((Function->FunctionFlags & FUNC_Native) == 0 && (Function->Script.Num() == 0))
		{
			return;
		}
	}

	#if WITH_EDITOR
	static const FName CallInEditorMeta("CallInEditor");
	const bool bAllowScriptExecution = GAllowActorScriptExecutionInEditor || 
		Function->GetBoolMetaData(CallInEditorMeta) ||
		Function->HasAnyFunctionFlags(FUNC_UbergraphFunction); // uber graph frame is only used internally, e.g. by timer handlers
	#else
	const bool bAllowScriptExecution = GAllowActorScriptExecutionInEditor;
	#endif
	UWorld* MyWorld = GetWorld();
	if( ((MyWorld && (MyWorld->AreActorsInitialized() || bAllowScriptExecution)) || HasAnyFlags(RF_ClassDefaultObject)) && !IsGarbageCollecting() )
	{
#if !UE_BUILD_SHIPPING
		if (!ProcessEventDelegate.IsBound() || !ProcessEventDelegate.Execute(this, Function, Parameters))
		{
			Super::ProcessEvent(Function, Parameters);
		}
#else
		Super::ProcessEvent(Function, Parameters);
#endif
	}
}

#if WITH_EDITOR
template <class F>
static bool ForEachStreamingRelevantComponent(const AActor* InActor, bool bForEditor, F Func)
{
	auto GetComponentStreamingBounds = [InActor, bForEditor](const UActorComponent* InComponent, FBox& OutStreamingBounds) -> bool
	{
		check(InActor);
		check(InComponent);

		if (!InComponent->IsRegistered())
		{
			return false;
		}

		// Transient components shouldn't be part of the streaming bounds, unless the actor itself is transient.
		// This is to allow transient actors to be loaded in PIE.
		if (!InActor->HasAnyFlags(RF_Transient) && InComponent->HasAnyFlags(RF_Transient))
		{
			return false;
		}

		// Editor-only components shouldn't be part of the streaming bounds, unless the actor itself is editor-only.
		// This is to allow editor-only actors to be loaded in PIE.
		if (!bForEditor && !InActor->IsEditorOnly() && InComponent->IsEditorOnly())
		{
			return false;
		}

		OutStreamingBounds = bForEditor ? InComponent->GetStreamingBoundsEditor() : InComponent->GetStreamingBounds();
		return !!OutStreamingBounds.IsValid;
	};

	auto HandleComponent = [&GetComponentStreamingBounds, &Func](const UActorComponent* Component) -> bool
	{
		FBox ComponentStreamingBound;
		if (GetComponentStreamingBounds(Component, ComponentStreamingBound))
		{
			Func(Component, ComponentStreamingBound);
			return true;
		}
		return false;
	};

	bool bHasStreamingRelevantComponents = false;

	InActor->ForEachComponent<UPrimitiveComponent>(true, [&bHasStreamingRelevantComponents, &HandleComponent](UActorComponent* Component)
	{
		bHasStreamingRelevantComponents |= HandleComponent(Component);
	});

	if (bForEditor || !bHasStreamingRelevantComponents)
	{
		InActor->ForEachComponent<UActorComponent>(false, [&bHasStreamingRelevantComponents, &HandleComponent](UActorComponent* Component)
		{
			// We already handled UPrimitive components.
			if (!Cast<UPrimitiveComponent>(Component))
			{
				bHasStreamingRelevantComponents |= HandleComponent(Component);
			}
		});
	}

	return bHasStreamingRelevantComponents;
}

static bool HasComponentForceActorNonSpatiallyLoaded(const AActor* InActor)
{
	bool bHasComponentForceActorNonSpatiallyLoaded = false;
	ForEachStreamingRelevantComponent(InActor, false, [&bHasComponentForceActorNonSpatiallyLoaded](const UActorComponent* Component, const FBox& StreamingBound)
	{
		bHasComponentForceActorNonSpatiallyLoaded |= Component->ForceActorNonSpatiallyLoaded();
	});
	return bHasComponentForceActorNonSpatiallyLoaded;
}

void AActor::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	OutRuntimeBounds.Init();
	if (!ForEachStreamingRelevantComponent(this, false, [&OutRuntimeBounds](const UActorComponent* Component, const FBox& StreamingBound)
	{
		OutRuntimeBounds += StreamingBound;
	}))
	{
		const FVector ActorLocation = GetActorLocation();
		OutRuntimeBounds = FBox(ActorLocation, ActorLocation);
	}

	OutEditorBounds.Init();
	ForEachStreamingRelevantComponent(this, true, [&OutEditorBounds](const UActorComponent* Component, const FBox& StreamingBound)
	{
		OutEditorBounds += StreamingBound;
	});
}

FBox AActor::GetStreamingBounds() const
{
	FBox StreamingBounds(ForceInit);
	ForEachStreamingRelevantComponent(this, false, [&StreamingBounds](const UActorComponent* Component, const FBox& StreamingBound)
	{
		StreamingBounds += StreamingBound;
	});
	return StreamingBounds;
}

bool AActor::GetIsSpatiallyLoaded() const
{
	return bIsSpatiallyLoaded && !HasComponentForceActorNonSpatiallyLoaded(this);
}
	
bool AActor::CanChangeIsSpatiallyLoadedFlag() const
{
	return !HasComponentForceActorNonSpatiallyLoaded(this);
}
#endif

void AActor::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	// Attached components will be shifted by parents, will shift only USceneComponents derived components
	if (RootComponent != nullptr && RootComponent->GetAttachParent() == nullptr)
	{
		RootComponent->ApplyWorldOffset(InOffset, bWorldShift);
	}

	// Shift UActorComponent derived components, but not USceneComponents
 	for (UActorComponent* ActorComponent : GetComponents())
 	{
 		if (IsValid(ActorComponent) && !ActorComponent->IsA<USceneComponent>())
 		{
 			ActorComponent->ApplyWorldOffset(InOffset, bWorldShift);
 		}
 	}
 	
	// Navigation receives update during component registration. World shift needs a separate path to shift all navigation data
	// So this normally should happen only in the editor when user moves visible sub-levels
	if (!bWorldShift && !InOffset.IsZero())
	{
		if (RootComponent != nullptr && RootComponent->IsRegistered())
		{
			FNavigationSystem::OnActorBoundsChanged(*this);
			FNavigationSystem::UpdateActorAndComponentData(*this);
		}
	}
}

/** Thread safe container for actor related global variables */
class FActorThreadContext : public TThreadSingleton<FActorThreadContext>
{
	friend TThreadSingleton<FActorThreadContext>;

	FActorThreadContext()
		: TestRegisterTickFunctions(nullptr)
	{
	}
public:
	/** Tests tick function registration */
	AActor* TestRegisterTickFunctions;
};

void AActor::RegisterActorTickFunctions(bool bRegister)
{
	check(!IsTemplate());

	if(bRegister)
	{
		if(PrimaryActorTick.bCanEverTick)
		{
			PrimaryActorTick.Target = this;
			PrimaryActorTick.SetTickFunctionEnable(PrimaryActorTick.bStartWithTickEnabled || PrimaryActorTick.IsTickFunctionEnabled());
			PrimaryActorTick.RegisterTickFunction(GetLevel());
		}
	}
	else
	{
		if(PrimaryActorTick.IsTickFunctionRegistered())
		{
			PrimaryActorTick.UnRegisterTickFunction();			
		}
	}

	FActorThreadContext::Get().TestRegisterTickFunctions = this; // we will verify the super call chain is intact. Don't copy and paste this to another actor class!
}

void AActor::RegisterAllActorTickFunctions(bool bRegister, bool bDoComponents)
{
	if(!IsTemplate())
	{
		// Prevent repeated redundant attempts
		if (bTickFunctionsRegistered != bRegister)
		{
			FActorThreadContext& ThreadContext = FActorThreadContext::Get();
			check(ThreadContext.TestRegisterTickFunctions == nullptr);
			RegisterActorTickFunctions(bRegister);
			bTickFunctionsRegistered = bRegister;
			checkf(ThreadContext.TestRegisterTickFunctions == this, TEXT("Failed to route Actor RegisterTickFunctions (%s)"), *GetFullName());
			ThreadContext.TestRegisterTickFunctions = nullptr;
		}

		if (bDoComponents)
		{
			for (UActorComponent* Component : GetComponents())
			{
				if (Component)
				{
					Component->RegisterAllComponentTickFunctions(bRegister);
				}
			}
		}

		if (bAsyncPhysicsTickEnabled)
		{
			if (UWorld* World = GetWorld())
			{
				if (FPhysScene_Chaos* Scene = static_cast<FPhysScene_Chaos*>(World->GetPhysicsScene()))
				{
					if (bRegister)
					{
						Scene->RegisterAsyncPhysicsTickActor(this);
					}
					else
					{
						Scene->UnregisterAsyncPhysicsTickActor(this);
					}
				}
			}
		}
	}
}

void AActor::SetActorTickEnabled(bool bEnabled)
{
	if (PrimaryActorTick.bCanEverTick && !IsTemplate())
	{
		PrimaryActorTick.SetTickFunctionEnable(bEnabled);
	}
}

bool AActor::IsActorTickEnabled() const
{
	return PrimaryActorTick.IsTickFunctionEnabled();
}

void AActor::SetActorTickInterval(float TickInterval)
{
	PrimaryActorTick.TickInterval = TickInterval;
}

float AActor::GetActorTickInterval() const
{
	return PrimaryActorTick.TickInterval;
}

bool AActor::IsMainPackageActor() const
{
	return IsPackageExternal() && ParentComponent.IsExplicitlyNull();
}

AActor* AActor::FindActorInPackage(UPackage* InPackage, bool bEvenIfPendingKill)
{
	AActor* Actor = nullptr;
	ForEachObjectWithPackage(InPackage, [&Actor](UObject* Object)
	{
		if (AActor* CurrentActor = Cast<AActor>(Object))
		{
			if (CurrentActor->IsMainPackageActor())
			{
				Actor = CurrentActor;
			}
		}
		return !Actor;
	}, EGetObjectsFlags::None, bEvenIfPendingKill ? RF_NoFlags : RF_MirroredGarbage);
	return Actor;
}

void AActor::PreSaveFromRoot(FObjectPreSaveRootContext ObjectSaveContext)
{
#if WITH_EDITOR
	if (ObjectSaveContext.IsCooking())
	{
		if (UPackage* ActorExternalPackage = GetExternalPackage())
		{
			DetachExternalPackage();

			// Process objects found in the actor package
			TArray<UObject*> Objects;
			GetObjectsWithPackage(ActorExternalPackage, Objects, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);

			UPackage* ActorPackage = GetPackage();
			for (UObject* Object : Objects)
			{
				if (!Cast<AActor>(Object) && Object->GetFName() != NAME_PackageMetaData)
				{
					// Move objects in the destination actor package
					Object->Rename(nullptr, ActorPackage);

					// Keep track of which package to reattach later
					ObjectsDetachedFromExternalPackage.Add(Object);
				}
			}
		}
	}
#endif
}

void AActor::PostSaveFromRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
#if WITH_EDITOR
	if (ObjectSaveContext.IsCooking())
	{
		ReattachExternalPackage();

		if (UPackage* ActorExternalPackage = GetExternalPackage())
		{
			// Reouter objects previously found in the actors packages to their original packages
			for (const TObjectPtr<UObject>& ObjectToReattach : ObjectsDetachedFromExternalPackage)
			{
				ObjectToReattach->Rename(nullptr, ActorExternalPackage);
			}
		}

		ObjectsDetachedFromExternalPackage.Empty();
	}
#endif
}

bool AActor::Rename( const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags )
{
	const bool bRenameTest = ((Flags & REN_Test) != 0);
	const bool bPerformComponentRegWork = ((Flags & REN_SkipComponentRegWork) == 0);
	const bool bChangingOuters = (NewOuter && (NewOuter != GetOuter()));

#if WITH_EDITOR
	// if we have an external actor and the actor is changing name/outer, we will want to update its package
	const bool bExternalActor = IsPackageExternal();
	const bool bIsPIEPackage = bExternalActor ? !!(GetExternalPackage()->GetPackageFlags() & PKG_PlayInEditor) : false;
	bool bShouldSetPackageExternal = false;
	if (!bRenameTest)
	{
		if (ULevel* MyLevel = GetLevel())
		{
			if (bExternalActor)
			{
				bShouldSetPackageExternal = bChangingOuters || IsMainPackageActor();
				// If we are changing outers always change the external package because the outer chain has an impact on the filename
				// but if we are not changing outers only change the external flag if the actor is the main actor in a package. (this is to avoid Child Actors creating packages)
				if (bShouldSetPackageExternal && !bIsPIEPackage)
				{
					bool bLevelPackageWasDirty = MyLevel->GetPackage()->IsDirty();
					SetPackageExternal(false, MyLevel->IsUsingExternalActors());

					// If we are not changing the outer, SetPackageExternal will still mark dirty the new - temporary - outer (which is the level)
					// Restore its backed-up dirty state
					if (!bChangingOuters && !bLevelPackageWasDirty)
					{
						MyLevel->GetPackage()->SetDirtyFlag(false);
					}
				}
			}
			if (bChangingOuters && MyLevel->IsUsingActorFolders())
			{
				// When changing outer, resolve FolderPath.
				// New level will handle conversion from path to actor folder guid if necessary.
				FolderPath = GetFolderPath();
				FolderGuid.Invalidate();
			}
		}
	}
#endif

	if (!bRenameTest && bChangingOuters)
	{
		RegisterAllActorTickFunctions(false, true); // unregister all tick functions

		if (bPerformComponentRegWork)
		{
			UnregisterAllComponents();
		}

		if (ULevel* MyLevel = GetLevel())
		{
			int32 ActorIndex;
			if (MyLevel->Actors.Find(this, ActorIndex))
			{
				MyLevel->Actors[ActorIndex] = nullptr;
				MyLevel->ActorsForGC.Remove(this);
				// TODO: There may need to be some consideration about removing this actor from the level cluster, but that would probably require destroying the entire cluster, so defer for now
			}
		}
	}

#if WITH_EDITOR
	UObject* OldOuter = GetOuter();
#endif

	const bool bSuccess = Super::Rename( InName, NewOuter, Flags );

#if WITH_EDITOR
	// Restore external package state, except when in PIE since it's transient and serves no purpose
	if (bShouldSetPackageExternal && !bIsPIEPackage)
	{
		if (ULevel* MyLevel = GetLevel())
		{
			// Don't dirty the current package (which is the level package) if the outer doesn't change
			check(bChangingOuters || (MyLevel->GetPackage() == GetPackage()));
			SetPackageExternal(true, bChangingOuters && MyLevel->IsUsingExternalActors());
		}
	}
#endif

	if (!bRenameTest && bChangingOuters)
	{
		if (ULevel* MyLevel = GetLevel())
		{
			MyLevel->TryAddActorToList(this, /*bAddUnique*/false);

			UWorld* World = MyLevel->GetWorld();
			if (World && World->bIsWorldInitialized && bPerformComponentRegWork)
			{
				RegisterAllComponents();
			}
			RegisterAllActorTickFunctions(true, true); // register all tick functions
#if WITH_EDITOR
			// If new outer level uses actor folder objects, update folder path (this will create actor folder if necessary an update actor's FolderGuid)
			if (MyLevel->IsUsingActorFolders())
			{
				SetFolderPath(FolderPath);
			}
#endif
		}

#if WITH_EDITOR
		if (GEngine && OldOuter != GetOuter())
		{
			GEngine->BroadcastLevelActorOuterChanged(this, OldOuter);
		}
#endif
	}
	return bSuccess;
}

UNetConnection* AActor::GetNetConnection() const
{
	return Owner ? Owner->GetNetConnection() : nullptr;
}

UPlayer* AActor::GetNetOwningPlayerAnyRole()
{
	if (Owner)
	{
		return Owner->GetNetOwningPlayer();
	}
	return nullptr;
}

UPlayer* AActor::GetNetOwningPlayer()
{
	// We can only replicate RPCs to the owning player
	if (GetLocalRole() == ROLE_Authority)
	{
		if (Owner)
		{
			return Owner->GetNetOwningPlayer();
		}
	}
	return nullptr;
}

bool AActor::DestroyNetworkActorHandled()
{
	return false;
}

void AActor::TickActor( float DeltaSeconds, ELevelTick TickType, FActorTickFunction& ThisTickFunction )
{
	// Actor validity was checked before this
	if (GetWorld())
	{
		Tick(DeltaSeconds);	// perform any tick functions unique to an actor subclass
	}
}

void AActor::Tick( float DeltaSeconds )
{
	if (GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint) || !GetClass()->HasAnyClassFlags(CLASS_Native))
	{
		// Allow tick if we are not a dedicated server, or we allow this tick on dedicated servers
		if (bAllowReceiveTickEventOnDedicatedServer || !IsRunningDedicatedServer())
		{
			ReceiveTick(DeltaSeconds);
		}

		// Update any latent actions we have for this actor
		// If this tick is skipped on a frame because we've got a TickInterval, our latent actions will be ticked
		// anyway by UWorld::Tick(). Given that, our latent actions don't need to be passed a larger
		// DeltaSeconds to make up the frames that they missed (because they wouldn't have missed any).
		// So pass in the world's DeltaSeconds value rather than our specific DeltaSeconds value.
		UWorld* MyWorld = GetWorld();
		if (MyWorld)
		{
			FLatentActionManager& LatentActionManager = MyWorld->GetLatentActionManager();
			LatentActionManager.ProcessLatentActions(this, MyWorld->GetDeltaSeconds());
		}
	}
}

/** If true, actor is ticked even if TickType==LEVELTICK_ViewportsOnly */
bool AActor::ShouldTickIfViewportsOnly() const
{
	return false;
}

void AActor::PreReplication(IRepChangedPropertyTracker & ChangedPropertyTracker)
{
#if WITH_PUSH_MODEL
	const AActor* const OldAttachParent = AttachmentReplication.AttachParent;
	const UActorComponent* const OldAttachComponent = AttachmentReplication.AttachComponent;
#endif

	// Attachment replication gets filled in by GatherCurrentMovement(), but in the case of a detached root we need to trigger remote detachment.
	AttachmentReplication.AttachParent = nullptr;
	AttachmentReplication.AttachComponent = nullptr;

	GatherCurrentMovement();

	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(AActor, ReplicatedMovement, IsReplicatingMovement());

	// Don't need to replicate AttachmentReplication if the root component replicates, because it already handles it.
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(AActor, AttachmentReplication, RootComponent && !RootComponent->GetIsReplicated());


#if WITH_PUSH_MODEL
	if (UNLIKELY(OldAttachParent != AttachmentReplication.AttachParent || OldAttachComponent != AttachmentReplication.AttachComponent))
	{
		MARK_PROPERTY_DIRTY_FROM_NAME(AActor, AttachmentReplication, this);
	}
#endif
}

void AActor::CallPreReplication(UNetDriver* NetDriver)
{
	if (NetDriver == nullptr)
	{
		return;
	}
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET(this, GetClass());
	
	const bool bPreReplication = ShouldCallPreReplication();
	const bool bPreReplicationForReplay = ShouldCallPreReplicationForReplay();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// This interface is obsolete and will be removed or re-purposed and renamed to pass parameters to PreReplication.
	IRepChangedPropertyTracker DummyRepChangedPropertyTracker;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Allow lazy creation/find of FRepChangedPropertyTracker
	UE::Net::Private::FNetPropertyConditionManager::FAllowCreateTrackerFromSetPropertyActiveOverrideScope AllowCreateTracker(UE::Net::Private::FNetPropertyConditionManager::Get());	

	if (bPreReplication)
	{
		const ENetRole LocalRole = GetLocalRole();
		const UWorld* World = GetWorld();

		// PreReplication is only called on the server, except when we're recording a Client Replay.
		// In that case we call PreReplication on the locally controlled Character as well.
		const bool bRoleReplicates = (LocalRole == ROLE_Authority) || NetDriver->IsReplicationOfActorsWithAnyRoleEnabled();
		if (bRoleReplicates || ((LocalRole == ROLE_AutonomousProxy) && World && World->IsRecordingClientReplay()))
		{
			PreReplication(DummyRepChangedPropertyTracker);
		}
	}

	if (bPreReplicationForReplay)
	{
		// If we're recording a replay, call this for everyone (includes SimulatedProxies).
		if (Cast<UDemoNetDriver>(NetDriver) || NetDriver->HasReplayConnection())
		{
			PreReplicationForReplay(DummyRepChangedPropertyTracker);
		}
	}

	if (bPreReplication)
	{
		// Call PreReplication on all owned components that are replicated
		for (UActorComponent* Component : ReplicatedComponents)
		{
			// Only call on components that aren't pending kill
			if (IsValid(Component))
			{
				Component->PreReplication(DummyRepChangedPropertyTracker);
			}
		}
	}
}

void AActor::PreReplicationForReplay(IRepChangedPropertyTracker & ChangedPropertyTracker)
{
	GatherCurrentMovement();
}

void AActor::RewindForReplay()
{
}

void AActor::PostActorCreated()
{
	// nothing at the moment
}

void AActor::GetComponentsBoundingCylinder(float& OutCollisionRadius, float& OutCollisionHalfHeight, bool bNonColliding, bool bIncludeFromChildActors) const
{
	bool bIgnoreRegistration = false;

#if WITH_EDITOR
	if (IsTemplate())
	{
		// Editor code calls this function on default objects when placing them in the viewport, so no components will be registered in those cases.
		UWorld* MyWorld = GetWorld();
		if (!MyWorld || !MyWorld->IsGameWorld())
		{
			bIgnoreRegistration = true;
		}
		else
		{
			UE_LOGF(LogActor, Log, "WARNING AActor::GetComponentsBoundingCylinder : Called on default object '%ls'. Will likely return zero size.", *this->GetPathName());
		}
	}
#elif !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (IsTemplate())
	{
		UE_LOGF(LogActor, Log, "WARNING AActor::GetComponentsBoundingCylinder : Called on default object '%ls'. Will likely return zero size.", *this->GetPathName());
	}
#endif

	OutCollisionRadius = 0.f;
	OutCollisionHalfHeight = 0.f;

	ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [&](const UPrimitiveComponent* InPrimComp)
	{
		// Only use collidable components to find collision bounding cylinder
		if ((bIgnoreRegistration || InPrimComp->IsRegistered()) && (bNonColliding || InPrimComp->IsCollisionEnabled()))
		{
			float TestRadius, TestHalfHeight;
			InPrimComp->CalcBoundingCylinder(TestRadius, TestHalfHeight);
			OutCollisionRadius = FMath::Max(OutCollisionRadius, TestRadius);
			OutCollisionHalfHeight = FMath::Max(OutCollisionHalfHeight, TestHalfHeight);
		}
	});
}

void AActor::GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const
{
	if (IsRootComponentCollisionRegistered())
	{
		RootComponent->CalcBoundingCylinder(CollisionRadius, CollisionHalfHeight);
	}
	else
	{
		GetComponentsBoundingCylinder(CollisionRadius, CollisionHalfHeight, false);
	}
}

bool AActor::IsRootComponentCollisionRegistered() const
{
	return RootComponent != nullptr && RootComponent->IsRegistered() && RootComponent->IsCollisionEnabled();
}

bool AActor::IsAttachedTo(const AActor* Other) const
{
	return (RootComponent && Other && Other->RootComponent) ? RootComponent->IsAttachedTo(Other->RootComponent) : false;
}

bool AActor::IsBasedOnActor(const AActor* Other) const
{
	return IsAttachedTo(Other);
}

#if WITH_EDITOR
bool AActor::Modify( bool bAlwaysMarkDirty/*=true*/ )
{
	if (!CanModify())
	{
		return false;
	}

	if (UChildActorComponent::ExperimentalAllowPerInstanceChildActorProperties())
	{
		if (AActor* ParentActor = GetParentActor())
		{
			return ParentActor->Modify(bAlwaysMarkDirty);
		}
	}

	// Any properties that reference a blueprint constructed component needs to avoid creating a reference to the component from the transaction
	// buffer, so we temporarily switch the property to non-transactional while the modify occurs
	TArray<FObjectProperty*> TemporarilyNonTransactionalProperties;
	if (GUndo)
	{
		for (TFieldIterator<FObjectProperty> PropertyIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FObjectProperty* ObjProp = *PropertyIt;
			if (!ObjProp->HasAllPropertyFlags(CPF_NonTransactional))
			{
				UActorComponent* ActorComponent = Cast<UActorComponent>(ObjProp->GetObjectPropertyValue(ObjProp->ContainerPtrToValuePtr<void>(this)));
				if (ActorComponent && ActorComponent->IsCreatedByConstructionScript())
				{
					ObjProp->SetPropertyFlags(CPF_NonTransactional);
					TemporarilyNonTransactionalProperties.Add(ObjProp);
				}
			}
		}
	}

	bool bSavedToTransactionBuffer = UObject::Modify( bAlwaysMarkDirty );

	for (FObjectProperty* ObjProp : TemporarilyNonTransactionalProperties)
	{
		ObjProp->ClearPropertyFlags(CPF_NonTransactional);
	}

	// If the root component is blueprint constructed we don't save it to the transaction buffer
	if (RootComponent)
	{
		if (!RootComponent->IsCreatedByConstructionScript())
		{
			bSavedToTransactionBuffer = RootComponent->Modify(bAlwaysMarkDirty) || bSavedToTransactionBuffer;
		}

		USceneComponent* DefaultAttachComp = GetDefaultAttachComponent();
		if (DefaultAttachComp && DefaultAttachComp != RootComponent && !DefaultAttachComp->IsCreatedByConstructionScript())
		{
			bSavedToTransactionBuffer = DefaultAttachComp->Modify(bAlwaysMarkDirty) || bSavedToTransactionBuffer;
		}
	}

	return bSavedToTransactionBuffer;
}
#endif

FBox AActor::GetComponentsBoundingBox(bool bNonColliding, bool bIncludeFromChildActors) const
{
	FBox Box(ForceInit);

	ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [&](const UPrimitiveComponent* InPrimComp)
	{
		// Only use collidable components to find collision bounding box.
		if (InPrimComp->IsRegistered() && (bNonColliding || InPrimComp->IsCollisionEnabled()))
		{
			Box += InPrimComp->Bounds.GetBox();
		}
	});

	return Box;
}

FBox AActor::CalculateComponentsBoundingBoxInLocalSpace(bool bNonColliding, bool bIncludeFromChildActors) const
{
	FBox Box(ForceInit);
	const FTransform& ActorToWorld = GetTransform();
	const FTransform WorldToActor = ActorToWorld.Inverse();

	ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [&](const UPrimitiveComponent* InPrimComp)
	{
		// Only use collidable components to find collision bounding box.
		if (InPrimComp->IsRegistered() && (bNonColliding || InPrimComp->IsCollisionEnabled()))
		{
			const FTransform ComponentToActor = InPrimComp->GetComponentTransform() * WorldToActor;
			Box += InPrimComp->CalcBounds(ComponentToActor).GetBox();
		}
	});

	return Box;
}

bool AActor::CheckStillInWorld()
{
	if (!IsValidChecked(this))
	{
		return false;
	}
	UWorld* MyWorld = GetWorld();
	if (!MyWorld)
	{
		return false;
	}

	// Only authority or non-networked actors should be destroyed, otherwise misprediction can destroy something the server is intending to keep alive.
	if (!(HasAuthority() || GetLocalRole() == ROLE_None))
	{
		return true;
	}

	// check the variations of KillZ
	AWorldSettings* WorldSettings = MyWorld->GetWorldSettings( true );

	if (!WorldSettings->AreWorldBoundsChecksEnabled())
	{
		return true;
	}

	if( GetActorLocation().Z < WorldSettings->KillZ )
	{
		UDamageType const* const DmgType = WorldSettings->KillZDamageType ? WorldSettings->KillZDamageType->GetDefaultObject<UDamageType>() : GetDefault<UDamageType>();
		FellOutOfWorld(*DmgType);
		return false;
	}
	// Check if box has poked outside the world
	else if( ( RootComponent != nullptr ) && ( GetRootComponent()->IsRegistered() == true ) )
	{
		const FBox&	Box = GetRootComponent()->Bounds.GetBox();
		if(	Box.Min.X < -HALF_WORLD_MAX || Box.Max.X > HALF_WORLD_MAX ||
			Box.Min.Y < -HALF_WORLD_MAX || Box.Max.Y > HALF_WORLD_MAX ||
			Box.Min.Z < -HALF_WORLD_MAX || Box.Max.Z > HALF_WORLD_MAX )
		{
			UE_LOGF(LogActor, Warning, "%ls is outside the world bounds!", *GetName());
			OutsideWorldBounds();
			// not safe to use physics or collision at this point
			SetActorEnableCollision(false);
			DisableComponentsSimulatePhysics();
			return false;
		}
	}

	return true;
}

void AActor::SetTickGroup(ETickingGroup NewTickGroup)
{
	PrimaryActorTick.TickGroup = NewTickGroup;
}

void AActor::ClearComponentOverlaps()
{
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	if (!GetWorld()->IsActorTickAndUserCallbacksEnabled())
	{
		return;
	}
#endif

	TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
	GetComponents(PrimitiveComponents);

	// Remove owned components from overlap tracking
	// We don't traverse the RootComponent attachment tree since that might contain
	// components owned by other actors.
	TArray<FOverlapInfo, TInlineAllocator<3>> OverlapsForCurrentComponent;
	for (UPrimitiveComponent* const PrimComp : PrimitiveComponents)
	{
		OverlapsForCurrentComponent.Reset();
		OverlapsForCurrentComponent.Append(PrimComp->GetOverlapInfos());
		for (const FOverlapInfo& CurrentOverlap : OverlapsForCurrentComponent)
		{
			const bool bDoNotifies = true;
			const bool bSkipNotifySelf = false;
			PrimComp->EndComponentOverlap(CurrentOverlap, bDoNotifies, bSkipNotifySelf);
		}
	}
}

void AActor::UpdateOverlaps(bool bDoNotifies)
{
	// just update the root component, which will cascade down to the children
	USceneComponent* const RootComp = GetRootComponent();
	if (RootComp)
	{
		RootComp->UpdateOverlaps(nullptr, bDoNotifies);
	}
}

bool AActor::IsOverlappingActor(const AActor* Other) const
{
	for (UActorComponent* OwnedComp : OwnedComponents)
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(OwnedComp))
		{
			if ((PrimComp->GetOverlapInfos().Num() > 0) && PrimComp->IsOverlappingActor(Other))
			{
				// found one, finished
				return true;
			}
		}
	}
	return false;
}

void AActor::GetOverlappingActors(TArray<AActor*>& OutOverlappingActors, TSubclassOf<AActor> ClassFilter) const
{
	// prepare output
	TSet<AActor*> OverlappingActors;
	GetOverlappingActors(OverlappingActors, ClassFilter);

	OutOverlappingActors.Reset(OverlappingActors.Num());

	for (AActor* OverlappingActor : OverlappingActors)
	{
		OutOverlappingActors.Add(OverlappingActor);
	}
}

void AActor::GetOverlappingActors(TSet<AActor*>& OutOverlappingActors, TSubclassOf<AActor> ClassFilter) const
{
	// prepare output
	OutOverlappingActors.Reset();
	TSet<AActor*> OverlappingActorsForCurrentComponent;

	for(UActorComponent* OwnedComp : OwnedComponents)
	{
		if(UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(OwnedComp))
		{
			PrimComp->GetOverlappingActors(OverlappingActorsForCurrentComponent, ClassFilter);

			OutOverlappingActors.Reserve(OutOverlappingActors.Num() + OverlappingActorsForCurrentComponent.Num());

			// then merge it into our final list
			for (AActor* OverlappingActor : OverlappingActorsForCurrentComponent)
			{
				if (OverlappingActor != this)
				{
					OutOverlappingActors.Add(OverlappingActor);
				}
			}
		}
	}
}

void AActor::GetOverlappingComponents(TArray<UPrimitiveComponent*>& OutOverlappingComponents) const
{
	TSet<UPrimitiveComponent*> OverlappingComponents;
	GetOverlappingComponents(OverlappingComponents);

	OutOverlappingComponents.Reset(OverlappingComponents.Num());

	for (UPrimitiveComponent* OverlappingComponent : OverlappingComponents)
	{
		OutOverlappingComponents.Add(OverlappingComponent);
	}
}

void AActor::GetOverlappingComponents(TSet<UPrimitiveComponent*>& OutOverlappingComponents) const
{
	OutOverlappingComponents.Reset();
	TArray<UPrimitiveComponent*> OverlappingComponentsForCurrentComponent;

	for (UActorComponent* OwnedComp : OwnedComponents)
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(OwnedComp))
		{
			// get list of components from the component
			PrimComp->GetOverlappingComponents(OverlappingComponentsForCurrentComponent);

			OutOverlappingComponents.Reserve(OutOverlappingComponents.Num() + OverlappingComponentsForCurrentComponent.Num());

			// then merge it into our final list
			for (UPrimitiveComponent* OverlappingComponent : OverlappingComponentsForCurrentComponent)
			{
				OutOverlappingComponents.Add(OverlappingComponent);
			}
		}
	}
}

void AActor::NotifyActorBeginOverlap(AActor* OtherActor)
{
	// call BP handler
	ReceiveActorBeginOverlap(OtherActor);
}

void AActor::NotifyActorEndOverlap(AActor* OtherActor)
{
	// call BP handler
	ReceiveActorEndOverlap(OtherActor);
}

void AActor::NotifyActorBeginCursorOver()
{
	// call BP handler
	ReceiveActorBeginCursorOver();
}

void AActor::NotifyActorEndCursorOver()
{
	// call BP handler
	ReceiveActorEndCursorOver();
}

void AActor::NotifyActorOnClicked(FKey ButtonPressed)
{
	// call BP handler
	ReceiveActorOnClicked(ButtonPressed);
}

void AActor::NotifyActorOnReleased(FKey ButtonReleased)
{
	// call BP handler
	ReceiveActorOnReleased(ButtonReleased);
}

void AActor::NotifyActorOnInputTouchBegin(const ETouchIndex::Type FingerIndex)
{
	// call BP handler
	ReceiveActorOnInputTouchBegin(FingerIndex);
}

void AActor::NotifyActorOnInputTouchEnd(const ETouchIndex::Type FingerIndex)
{
	// call BP handler
	ReceiveActorOnInputTouchEnd(FingerIndex);
}

void AActor::NotifyActorOnInputTouchEnter(const ETouchIndex::Type FingerIndex)
{
	// call BP handler
	ReceiveActorOnInputTouchEnter(FingerIndex);
}

void AActor::NotifyActorOnInputTouchLeave(const ETouchIndex::Type FingerIndex)
{
	// call BP handler
	ReceiveActorOnInputTouchLeave(FingerIndex);
}


void AActor::NotifyHit(class UPrimitiveComponent* MyComp, AActor* Other, class UPrimitiveComponent* OtherComp, bool bSelfMoved, FVector HitLocation, FVector HitNormal, FVector NormalImpulse, const FHitResult& Hit)
{
	// call BP handler
	ReceiveHit(MyComp, Other, OtherComp, bSelfMoved, HitLocation, HitNormal, NormalImpulse, Hit);
}

/** marks all PrimitiveComponents for which their Owner is relevant for visibility as dirty because the Owner of some Actor in the chain has changed
 * @param TheActor the actor to mark components dirty for
 */
static void MarkOwnerRelevantComponentsDirty(AActor* TheActor)
{
	for (UActorComponent* Component : TheActor->GetComponents())
	{
		UPrimitiveComponent* Primitive = Cast<UPrimitiveComponent>(Component);
		if (Primitive && Primitive->IsRegistered() && (Primitive->bOnlyOwnerSee || Primitive->bOwnerNoSee))
		{
			Primitive->MarkRenderStateDirty();
		}
	}

	// recurse over children of this Actor
	for (int32 i = 0; i < TheActor->Children.Num(); i++)
	{
		AActor* Child = TheActor->Children[i];
		if (IsValid(Child))
		{
			MarkOwnerRelevantComponentsDirty(Child);
		}
	}
}

bool AActor::WasRecentlyRendered(float Tolerance) const
{
	if (const UWorld* const World = GetWorld())
	{
		// Adjust tolerance, so visibility is not affected by bad frame rate / hitches.
		const float RenderTimeThreshold = FMath::Max(Tolerance, World->DeltaTimeSeconds + UE_KINDA_SMALL_NUMBER);

		// If the current cached value is less than the tolerance then we don't need to go look at the components
		return World->TimeSince(GetLastRenderTime()) <= RenderTimeThreshold;
	}
	return false;
}

float AActor::GetLastRenderTime() const
{
	if (LastRenderTime.HasAlwaysVisibleComponents())
	{
		if (const UWorld* World = GetWorld())
		{
			return World->GetTimeSeconds();
		}
	}
	return LastRenderTime.GetLastRenderTime();
}

void AActor::SetOwner(AActor* NewOwner)
{
	if (Owner != NewOwner && IsValidChecked(this))
	{
		if (NewOwner != nullptr && NewOwner->IsOwnedBy(this))
		{
			UE_LOGF(LogActor, Error, "SetOwner(): Failed to set '%ls' owner of '%ls' because it would cause an Owner loop", *NewOwner->GetName(), *GetName());
			return;
		}

		// Sets this actor's parent to the specified actor.
		if (Owner != nullptr)
		{
			// remove from old owner's Children array
			verifySlow(Owner->Children.Remove(this) == 1);
		}

		Owner = NewOwner;
		MARK_PROPERTY_DIRTY_FROM_NAME(AActor, Owner, this);

		if (Owner != nullptr)
		{
			// add to new owner's Children array
			checkSlow(!Owner->Children.Contains(this));
			Owner->Children.Add(this);
		}

		// mark all components for which Owner is relevant for visibility to be updated
		if (bHasFinishedSpawning)
		{
			MarkOwnerRelevantComponentsDirty(this);
		}

		UpdateOwningNetConnection();
	}
}

bool AActor::HasLocalNetOwner() const
{
	// I might be the top owner if I am a Pawn or a Controller (owner will be null)
	const AActor* TopOwner = this;

	if (Owner != nullptr)
	{
		// I have an owner so search that for the top owner
		for (TopOwner = Owner; TopOwner->Owner; TopOwner = TopOwner->Owner)
		{
		}
	}

	// Top owner will normally be a Pawn or a Controller
	if (const APawn* Pawn = Cast<APawn>(TopOwner))
	{
		return Pawn->IsLocallyControlled();
	}

	const AController* Controller = Cast<AController>(TopOwner);
	return Controller && Controller->IsLocalController();
}

bool AActor::HasNetOwner() const
{
	if (Owner == nullptr)
	{
		// all basic AActors are unable to call RPCs without special AActors as their owners (ie APlayerController)
		return false;
	}

	// Find the topmost actor in this owner chain
	AActor* TopOwner = nullptr;
	for (TopOwner = Owner; TopOwner->Owner; TopOwner = TopOwner->Owner)
	{
	}

	return TopOwner->HasNetOwner();
}

void AActor::SetAutoDestroyWhenFinished(bool bVal)
{
	bAutoDestroyWhenFinished = bVal;
	if (UWorld* MyWorld = GetWorld())
	{
		if(UAutoDestroySubsystem* AutoDestroySub = MyWorld->GetSubsystem<UAutoDestroySubsystem>())
		{
			if (bAutoDestroyWhenFinished && (HasActorBegunPlay() || IsActorBeginningPlay()))
			{
				AutoDestroySub->RegisterActor(this);
			}
			else
			{
				AutoDestroySub->UnregisterActor(this);
			}
		}
	}
}

void AActor::K2_AttachRootComponentTo(USceneComponent* InParent, FName InSocketName, EAttachLocation::Type AttachLocationType /*= EAttachLocation::KeepRelativeOffset */, bool bWeldSimulatedBodies /*=true*/)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (RootComponent && InParent)
	{
		RootComponent->K2_AttachTo(InParent, InSocketName, AttachLocationType, bWeldSimulatedBodies);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool AActor::K2_AttachToComponent(USceneComponent* Parent, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule, bool bWeldSimulatedBodies)
{
	return AttachToComponent(Parent, FAttachmentTransformRules(LocationRule, RotationRule, ScaleRule, bWeldSimulatedBodies), SocketName);
}

bool AActor::AttachToComponent(USceneComponent* Parent, const FAttachmentTransformRules& AttachmentRules, FName SocketName)
{
	if (RootComponent && Parent)
	{
		return RootComponent->AttachToComponent(Parent, AttachmentRules, SocketName);
	}

	return false;
}

void AActor::OnRep_AttachmentReplication()
{
	if (AttachmentReplication.AttachParent)
	{
		if (RootComponent)
		{
			USceneComponent* AttachParentComponent = (AttachmentReplication.AttachComponent ? ToRawPtr(AttachmentReplication.AttachComponent) : AttachmentReplication.AttachParent->GetRootComponent());

			if (AttachParentComponent)
			{
				RootComponent->SetRelativeLocation_Direct(AttachmentReplication.LocationOffset);
				RootComponent->SetRelativeRotation_Direct(AttachmentReplication.RotationOffset);
				RootComponent->SetRelativeScale3D_Direct(AttachmentReplication.RelativeScale3D);

				// If we're already attached to the correct Parent and Socket, then the update must be position only.
				// AttachToComponent would early out in this case.
				// Note, we ignore the special case for simulated bodies in AttachToComponent as AttachmentReplication shouldn't get updated
				// if the body is simulated (see AActor::GatherMovement).
				const bool bAlreadyAttached = (AttachParentComponent == RootComponent->GetAttachParent() && AttachmentReplication.AttachSocket == RootComponent->GetAttachSocketName() && AttachParentComponent->GetAttachChildren().Contains(RootComponent));
				if (bAlreadyAttached)
				{
					// Note, this doesn't match AttachToComponent, but we're assuming it's safe to skip physics (see comment above).
					RootComponent->UpdateComponentToWorld(EUpdateTransformFlags::SkipPhysicsUpdate, ETeleportType::None);
				}
				else
				{
					RootComponent->AttachToComponent(AttachParentComponent, FAttachmentTransformRules::KeepRelativeTransform, AttachmentReplication.AttachSocket);
				}
			}
		}
	}
	else
	{
		DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);

		// Handle the case where an object was both detached and moved on the server in the same frame.
		// Calling this extraneously does not hurt but will properly fire events if the movement state changed while attached.
		// This is needed because client side movement is ignored when attached
		if (IsReplicatingMovement())
		{
			OnRep_ReplicatedMovement();
		}
	}
}

void AActor::K2_AttachRootComponentToActor(AActor* InParentActor, FName InSocketName /*= NAME_None*/, EAttachLocation::Type AttachLocationType /*= EAttachLocation::KeepRelativeOffset */, bool bWeldSimulatedBodies /*=true*/)
{
	if (RootComponent && InParentActor)
	{
		USceneComponent* ParentDefaultAttachComponent = InParentActor->GetDefaultAttachComponent();
		if (ParentDefaultAttachComponent)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			RootComponent->K2_AttachTo(ParentDefaultAttachComponent, InSocketName, AttachLocationType, bWeldSimulatedBodies );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

bool AActor::K2_AttachToActor(AActor* ParentActor, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule, bool bWeldSimulatedBodies)
{
	return AttachToActor(ParentActor, FAttachmentTransformRules(LocationRule, RotationRule, ScaleRule, bWeldSimulatedBodies), SocketName);
}

bool AActor::AttachToActor(AActor* ParentActor, const FAttachmentTransformRules& AttachmentRules, FName SocketName)
{
	if (RootComponent && ParentActor)
	{
		USceneComponent* ParentDefaultAttachComponent = ParentActor->GetDefaultAttachComponent();
		if (ParentDefaultAttachComponent)
		{
			return RootComponent->AttachToComponent(ParentDefaultAttachComponent, AttachmentRules, SocketName);
		}
	}

	return false;
}

void AActor::DetachRootComponentFromParent(bool bMaintainWorldPosition)
{
	if(RootComponent)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RootComponent->DetachFromParent(bMaintainWorldPosition);
PRAGMA_ENABLE_DEPRECATION_WARNINGS		

		// Clear AttachmentReplication struct
		AttachmentReplication = FRepAttachment();
		MARK_PROPERTY_DIRTY_FROM_NAME(AActor, AttachmentReplication, this);
	}
}

void AActor::K2_DetachFromActor(EDetachmentRule LocationRule /*= EDetachmentRule::KeepRelative*/, EDetachmentRule RotationRule /*= EDetachmentRule::KeepRelative*/, EDetachmentRule ScaleRule /*= EDetachmentRule::KeepRelative*/)
{
	DetachFromActor(FDetachmentTransformRules(LocationRule, RotationRule, ScaleRule, true));
}

void AActor::DetachFromActor(const FDetachmentTransformRules& DetachmentRules)
{
	if (RootComponent)
	{
		RootComponent->DetachFromComponent(DetachmentRules);
	}
}

void AActor::DetachAllSceneComponents(USceneComponent* InParentComponent, const FDetachmentTransformRules& DetachmentRules)
{
	if (InParentComponent)
	{
		TInlineComponentArray<USceneComponent*> Components;
		GetComponents(Components);

		for (USceneComponent* SceneComp : Components)
		{
			if (SceneComp->GetAttachParent() == InParentComponent)
			{
				SceneComp->DetachFromComponent(DetachmentRules);
			}
		}
	}
}

AActor* AActor::GetAttachParentActor() const
{
	if (GetRootComponent() && GetRootComponent()->GetAttachParent())
	{
		return GetRootComponent()->GetAttachParent()->GetOwner();
	}

	return nullptr;
}

FName AActor::GetAttachParentSocketName() const
{
	if (GetRootComponent() && GetRootComponent()->GetAttachParent())
	{
		return GetRootComponent()->GetAttachSocketName();
	}

	return NAME_None;
}

void AActor::ForEachAttachedActors(TFunctionRef<bool(class AActor*)> Functor) const
{
	if (RootComponent != nullptr)
	{
		// Current set of components to check
		TInlineComponentArray<USceneComponent*> CompsToCheck;

		// Set of all components we have checked
		TInlineComponentArray<USceneComponent*> CheckedComps;

		CompsToCheck.Push(RootComponent);

		// While still work left to do
		while (CompsToCheck.Num() > 0)
		{
			// Get the next off the queue
			USceneComponent* SceneComp = CompsToCheck.Pop(EAllowShrinking::No);

			// Add it to the 'checked' set, should not already be there!
			CheckedComps.Add(SceneComp);

			AActor* CompOwner = SceneComp->GetOwner();
			if (CompOwner != nullptr)
			{
				if (CompOwner != this)
				{
					// If this component has a different owner, call the callback and stop if told.
					if (!Functor(CompOwner))
					{
						// The functor wants us to abort
						return;
					}
				}
				else
				{
					// This component is owned by us, we need to add its children
					for (USceneComponent* ChildComp : SceneComp->GetAttachChildren())
					{
						// Add any we have not explored yet to the set to check
						if (ChildComp != nullptr && !CheckedComps.Contains(ChildComp))
						{
							CompsToCheck.Push(ChildComp);
						}
					}
				}
			}
		}
	}
}

void AActor::GetAttachedActors(TArray<class AActor*>& OutActors, bool bResetArray, bool bRecursivelyIncludeAttachedActors) const
{
	if (bResetArray)
	{
		OutActors.Reset();
	}

	ForEachAttachedActors([bRecursivelyIncludeAttachedActors, &OutActors](AActor* AttachedActor)
	{
		int32 OriginalNumActors = OutActors.Num();

		if ((OriginalNumActors <= OutActors.AddUnique(AttachedActor)) && bRecursivelyIncludeAttachedActors)
		{
			AttachedActor->GetAttachedActors(OutActors, false, true);
		}
		return true;
	});



}

bool AActor::ActorHasTag(FName Tag) const
{
	return (Tag != NAME_None) && Tags.Contains(Tag);
}

bool AActor::IsInLevel(const ULevel *TestLevel) const
{
	return (GetLevel() == TestLevel);
}

ULevel* AActor::GetLevel() const
{
	return GetTypedOuter<ULevel>();
}

FTransform AActor::GetLevelTransform() const
{
	if (ULevel* Level = GetLevel())
	{
		if (ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(Level))
		{
			return LevelStreaming->LevelTransform;
		}
	}

	return FTransform::Identity;
}

bool AActor::IsInPersistentLevel(bool bIncludeLevelStreamingPersistent) const
{
	ULevel* MyLevel = GetLevel();
	UWorld* World = GetWorld();
	return ( (MyLevel == World->PersistentLevel) || ( bIncludeLevelStreamingPersistent && World->GetStreamingLevels().Num() > 0 &&
														Cast<ULevelStreamingPersistent>(World->GetStreamingLevels()[0]) &&
														World->GetStreamingLevels()[0]->GetLoadedLevel() == MyLevel ) );
}

bool AActor::IsRootComponentStatic() const
{
	return(RootComponent != nullptr && RootComponent->Mobility == EComponentMobility::Static);
}

bool AActor::IsRootComponentStationary() const
{
	return(RootComponent != nullptr && RootComponent->Mobility == EComponentMobility::Stationary);
}

bool AActor::IsRootComponentMovable() const
{
	return(RootComponent != nullptr && RootComponent->Mobility == EComponentMobility::Movable);
}

APhysicsVolume* AActor::GetPhysicsVolume() const
{
	if (const USceneComponent* ActorRootComponent = GetRootComponent())
	{
		return ActorRootComponent->GetPhysicsVolume();
	}

	return GetWorld()->GetDefaultPhysicsVolume();
}

FVector AActor::GetTargetLocation(AActor* RequestedBy) const
{
	return GetActorLocation();
}


bool AActor::IsRelevancyOwnerFor(const AActor* ReplicatedActor, const AActor* ActorOwner, const AActor* ConnectionActor) const
{
	return (ActorOwner == this);
}

void AActor::ForceNetUpdate()
{
#if CSV_RECORD_DETAILED_ACTOR_NET_STATS
	ActorUtils::CsvRecordClassName(GetClass(), CSV_CATEGORY_INDEX(Actor_ForceNetUpdate));
#endif // CSV_RECORD_DETAILED_ACTOR_NET_STATS

	UNetDriver* NetDriver = GetNetDriver();

	if (GetLocalRole() == ROLE_Authority)
	{
		// ForceNetUpdate on the game net driver only if we are the authority...
		if (NetDriver && NetDriver->GetNetMode() < ENetMode::NM_Client) // ... and not a client
		{
			NetDriver->ForceNetUpdate(this);

			if (NetDormancy > DORM_Awake)
			{
				FlushNetDormancy();
			}
		}
	}

	// Even if not authority, other drivers (like the demo net driver) may need to ForceNetUpdate
	if (UWorld* MyWorld = GetWorld())
	{
		if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(MyWorld))
		{
			for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
			{
				if (Driver.NetDriver != nullptr && Driver.NetDriver != NetDriver && Driver.NetDriver->ShouldReplicateActor(this))
				{
					Driver.NetDriver->ForceNetUpdate(this);
				}
			}
		}
	}
}

void AActor::SetNetDormancy(ENetDormancy NewDormancy)
{
	if (IsNetMode(NM_Client))
	{
		return;
	}

	if (IsPendingKillPending())
	{
		return;
	}

	ENetDormancy OldDormancy = NetDormancy;
	NetDormancy = NewDormancy;
	const bool bDormancyChanged = (OldDormancy != NewDormancy);

	if (UWorld* MyWorld = GetWorld())
	{
		if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(MyWorld))
		{
			// Tell driver about change
			if (bDormancyChanged)
			{
				for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
				{
					if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(this))
					{
						Driver.NetDriver->NotifyActorDormancyChange(this, OldDormancy);
					}
				}
			}

			// If not dormant, flush actor from NetDriver's dormant list
			if (NewDormancy <= DORM_Awake)
			{
				// Since we are coming out of dormancy, make sure we are on the network actor list
				MyWorld->AddNetworkActor(this);

				for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
				{
					if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(this))
					{
						Driver.NetDriver->FlushActorDormancy(this);
					}
				}
			}
		}
	}
}

/** Removes the actor from the NetDriver's dormancy list: forcing at least one more update. */
void AActor::FlushNetDormancy()
{
	if (IsNetMode(NM_Client) || NetDormancy <= DORM_Awake || IsPendingKillPending())
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(NET_AActor_FlushNetDormancy);

#if CSV_RECORD_DETAILED_ACTOR_NET_STATS
	ActorUtils::CsvRecordClassName(GetClass(), CSV_CATEGORY_INDEX(Actor_FlushNetDormancy));
#endif // CSV_RECORD_DETAILED_ACTOR_NET_STATS

	bool bWasDormInitial = false;
	if (NetDormancy == DORM_Initial)
	{
		// No longer initially dormant
		NetDormancy = DORM_DormantAll;
		bWasDormInitial = true;
	}

	// Don't proceed with network operations if not actually set to replicate
	if (!bReplicates)
	{
		return;
	}

	if (UWorld* const MyWorld = GetWorld())
	{
		// Add to network actors list if needed
		MyWorld->AddNetworkActor(this);
	
		if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(MyWorld))
		{
			for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
			{
				if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(this))
				{
					Driver.NetDriver->FlushActorDormancy(this, bWasDormInitial);
				}
			}
		}
	}
}

void AActor::ForcePropertyCompare()
{
	if (IsNetMode(NM_Client))
	{
		return;
	}

	if (!bReplicates)
	{
		return;
	}

	if (const UWorld* MyWorld = GetWorld())
	{
		if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(MyWorld))
		{
			for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
			{
				if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(this))
				{
					Driver.NetDriver->ForcePropertyCompare(this);
				}
			}
		}
	}
}

void AActor::PostRenderFor(APlayerController *PC, UCanvas *Canvas, FVector CameraPosition, FVector CameraDir) {}

void AActor::PrestreamTextures( float Seconds, bool bEnableStreaming, int32 CinematicTextureGroups )
{
	// This only handles non-location-based streaming. Location-based streaming is handled by SeqAct_StreamInTextures::UpdateOp.
	float Duration = 0.0;
	if ( bEnableStreaming )
	{
		// A Seconds==0.0f, it means infinite (e.g. 30 days)
		Duration = FMath::IsNearlyZero(Seconds) ? (60.0f*60.0f*24.0f*30.0f) : Seconds;
	}

	// Iterate over all components of that actor
	for (UActorComponent* Component : GetComponents())
	{
		// If its a static mesh component, with a static mesh
		UMeshComponent* MeshComponent = Cast<UMeshComponent>(Component);
		if (MeshComponent && MeshComponent->IsRegistered() )
		{
			MeshComponent->PrestreamTextures( Duration, false, CinematicTextureGroups );
		}
	}
}

void AActor::OnRep_Instigator() {}

void AActor::OnRep_ReplicateMovement()
{
	// If the actor stops replicating movement and is using PhysicsReplication PredictiveInterpolation, remove replicated target else it will linger waiting for a sleep state.
	if (!IsReplicatingMovement() && GetPhysicsReplicationMode() == EPhysicsReplicationMode::PredictiveInterpolation)
	{
		if (UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(RootComponent))
		{
			if (UWorld* World = GetWorld())
			{
				if (FPhysScene* PhysScene = World->GetPhysicsScene())
				{
					if (IPhysicsReplication* PhysicsReplication = PhysScene->GetPhysicsReplication())
					{
						PhysicsReplication->RemoveReplicatedTarget(RootPrimComp);
					}
				}
			}
		}
	}
}

void AActor::RouteEndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (bActorInitialized)
	{
		if (ActorHasBegunPlay == EActorBeginPlayState::HasBegunPlay)
		{
			SCOPE_CYCLE_UOBJECT(EndPlay, this);

			EndPlay(EndPlayReason);
			ensureMsgf(ActorHasBegunPlay == EActorBeginPlayState::HasNotBegunPlay, TEXT("EndPlay on %s failed. Make sure to call Super::EndPlay() in your override function."), *GetName());
		}

		// Behaviors specific to an actor being unloaded due to a streaming level removal
		if (EndPlayReason == EEndPlayReason::RemovedFromWorld)
		{
			ClearComponentOverlaps();

			bActorInitialized = false;
			if (UWorld* World = GetWorld())
			{
				World->RemoveNetworkActor(this);
				UE::Net::FReplicationSystemUtil::StopReplicatingActor(this, FStopReplicatingActorParams(EndPlayReason));
			}
		}

		// Clear any ticking lifespan timers
		if (TimerHandle_LifeSpanExpired.IsValid())
		{
			SetLifeSpan(0.f);
		}
	}

	{
		SCOPE_CYCLE_UOBJECT(UninitializeComponents, this);
		UninitializeComponents();
	}
}

void AActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (ActorHasBegunPlay == EActorBeginPlayState::HasBegunPlay)
	{
		TRACE_OBJECT_LIFETIME_END(this);

		ActorHasBegunPlay = EActorBeginPlayState::HasNotBegunPlay;

		// This must be called otherwise the ReplicationSystem will keep a reference to the actor forever.
		UE::Net::FReplicationSystemUtil::StopReplicatingActor(this, FStopReplicatingActorParams(EndPlayReason));

		// Dispatch the blueprint events
		ReceiveEndPlay(EndPlayReason);
		OnEndPlay.Broadcast(this, EndPlayReason);

		TInlineComponentArray<UActorComponent*> Components;
		GetComponents(Components);

		for (UActorComponent* Component : Components)
		{
			if (Component->HasBegunPlay())
			{
				Component->EndPlay(EndPlayReason);
				ensureMsgf(Component->HasBegunPlay() == false, TEXT("EndPlay on %s failed. Make sure to call Super::EndPlay() in the override function in class %s."), *Component->GetName(), *Component->GetClass()->GetName());
			}
		}
	}
}

FVector AActor::GetPlacementExtent() const
{
	FVector Extent(0.f);
	if( (RootComponent && GetRootComponent()->ShouldCollideWhenPlacing()) && bCollideWhenPlacing) 
	{
		FBox ActorBox(ForceInit);
		for (UActorComponent* Component : GetComponents())
		{
			USceneComponent* SceneComp = Cast<USceneComponent>(Component);
			if (SceneComp && SceneComp->ShouldCollideWhenPlacing())
			{
				ActorBox += SceneComp->GetPlacementExtent().GetBox();
			}
		}

		// Get box extent, adjusting for any difference between the center of the box and the actor pivot
		FVector AdjustedBoxExtent = ActorBox.GetExtent() - ActorBox.GetCenter();
		float CollisionRadius = FMath::Sqrt((AdjustedBoxExtent.X * AdjustedBoxExtent.X) + (AdjustedBoxExtent.Y * AdjustedBoxExtent.Y));
		Extent = FVector(CollisionRadius, CollisionRadius, AdjustedBoxExtent.Z);
	}
	return Extent;
}

void AActor::Destroyed()
{
	RouteEndPlay(EEndPlayReason::Destroyed);

	ReceiveDestroyed();
	OnDestroyed.Broadcast(this);
}

void AActor::SetCallPreReplication(bool bCall)
{
	bCallPreReplication = bCall;
}

bool AActor::ShouldCallPreReplication() const
{
	// The extra conditions here are related to custom property conditions and GatherMovement
	return bCallPreReplication || bReplicateMovement || (RootComponent && !RootComponent->GetIsReplicated());
}

void AActor::SetCallPreReplicationForReplay(bool bCall)
{
	bCallPreReplicationForReplay = bCall;
}

bool AActor::ShouldCallPreReplicationForReplay() const
{
	// The extra conditions here are related to custom property conditions and GatherMovement
	return bCallPreReplicationForReplay || bReplicateMovement || (RootComponent && !RootComponent->GetIsReplicated());
}

void AActor::TearOff()
{
	const ENetMode NetMode = GetNetMode();

	if (NetMode == NM_ListenServer || NetMode == NM_DedicatedServer)
	{
		bTearOff = true;
		MARK_PROPERTY_DIRTY_FROM_NAME(AActor, bTearOff, this);
		
		if (UWorld* MyWorld = GetWorld())
		{
			if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(MyWorld))
			{
				for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
				{
					if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(this))
					{
						Driver.NetDriver->NotifyActorTearOff(this);
					}
				}
			}
		}
	}
}

void AActor::TornOff() {}

void AActor::Reset()
{
	K2_OnReset();
}

void AActor::FellOutOfWorld(const UDamageType& dmgType)
{
	// Only authority or non-networked actors should be destroyed, otherwise misprediction can destroy something the server is intending to keep alive.
	if (HasAuthority() || GetLocalRole() == ROLE_None)
	{
		DisableComponentsSimulatePhysics();
		SetActorHiddenInGame(true);
		SetActorEnableCollision(false);
		Destroy();
	}
}

void AActor::MakeNoise(float Loudness, APawn* NoiseInstigator, FVector NoiseLocation, float MaxRange, FName Tag)
{
	NoiseInstigator = NoiseInstigator ? NoiseInstigator : GetInstigator();
	if ((GetNetMode() != NM_Client) && NoiseInstigator)
	{
		AActor::MakeNoiseDelegate.Execute(this, Loudness, NoiseInstigator
			, NoiseLocation.IsZero() ? GetActorLocation() : NoiseLocation
			, MaxRange
			, Tag);
	}
}

void AActor::MakeNoiseImpl(AActor* NoiseMaker, float Loudness, APawn* NoiseInstigator, const FVector& NoiseLocation, float MaxRange, FName Tag)
{
	check(NoiseMaker);

	UPawnNoiseEmitterComponent* NoiseEmitterComponent = NoiseInstigator->GetPawnNoiseEmitterComponent();
	if (NoiseEmitterComponent)
	{
		// Note: MaxRange and Tag are not supported for this legacy component. Use AISense_Hearing instead.
		NoiseEmitterComponent->MakeNoise( NoiseMaker, Loudness, NoiseLocation );
	}
}

void AActor::SetMakeNoiseDelegate(const FMakeNoiseDelegate& NewDelegate)
{
	if (NewDelegate.IsBound())
	{
		MakeNoiseDelegate = NewDelegate;
	}
}

float AActor::TakeDamage(float DamageAmount, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser)
{
	float ActualDamage = DamageAmount;

	UDamageType const* const DamageTypeCDO = DamageEvent.DamageTypeClass ? DamageEvent.DamageTypeClass->GetDefaultObject<UDamageType>() : GetDefault<UDamageType>();
	if (DamageEvent.IsOfType(FPointDamageEvent::ClassID))
	{
		// point damage event, pass off to helper function
		FPointDamageEvent* const PointDamageEvent = (FPointDamageEvent*) &DamageEvent;
		ActualDamage = InternalTakePointDamage(ActualDamage, *PointDamageEvent, EventInstigator, DamageCauser);

		// K2 notification for this actor
		if (ActualDamage != 0.f)
		{
			ReceivePointDamage(ActualDamage, DamageTypeCDO, PointDamageEvent->HitInfo.ImpactPoint, PointDamageEvent->HitInfo.ImpactNormal, PointDamageEvent->HitInfo.Component.Get(), PointDamageEvent->HitInfo.BoneName, PointDamageEvent->ShotDirection, EventInstigator, DamageCauser, PointDamageEvent->HitInfo);
			OnTakePointDamage.Broadcast(this, ActualDamage, EventInstigator, PointDamageEvent->HitInfo.ImpactPoint, PointDamageEvent->HitInfo.Component.Get(), PointDamageEvent->HitInfo.BoneName, PointDamageEvent->ShotDirection, DamageTypeCDO, DamageCauser);

			// Notify the component
			UPrimitiveComponent* const PrimComp = PointDamageEvent->HitInfo.Component.Get();
			if (PrimComp)
			{
				PrimComp->ReceiveComponentDamage(DamageAmount, DamageEvent, EventInstigator, DamageCauser);
			}
		}
	}
	else if (DamageEvent.IsOfType(FRadialDamageEvent::ClassID))
	{
		// radial damage event, pass off to helper function
		FRadialDamageEvent* const RadialDamageEvent = (FRadialDamageEvent*) &DamageEvent;
		ActualDamage = InternalTakeRadialDamage(ActualDamage, *RadialDamageEvent, EventInstigator, DamageCauser);

		// K2 notification for this actor
		if (ActualDamage != 0.f)
		{
			FHitResult const& Hit = (RadialDamageEvent->ComponentHits.Num() > 0) ? RadialDamageEvent->ComponentHits[0] : FHitResult();
			ReceiveRadialDamage(ActualDamage, DamageTypeCDO, RadialDamageEvent->Origin, Hit, EventInstigator, DamageCauser);
			OnTakeRadialDamage.Broadcast(this, ActualDamage, DamageTypeCDO, RadialDamageEvent->Origin, Hit, EventInstigator, DamageCauser);

			// add any desired physics impulses to our components
			for (int HitIdx = 0; HitIdx < RadialDamageEvent->ComponentHits.Num(); ++HitIdx)
			{
				FHitResult const& CompHit = RadialDamageEvent->ComponentHits[HitIdx];
				UPrimitiveComponent* const PrimComp = CompHit.Component.Get();
				if (PrimComp && PrimComp->GetOwner() == this)
				{
					PrimComp->ReceiveComponentDamage(DamageAmount, DamageEvent, EventInstigator, DamageCauser);
				}
			}
		}
	}

	// generic damage notifications sent for any damage
	// note we will broadcast these for negative damage as well
	if (ActualDamage != 0.f)
	{
		ReceiveAnyDamage(ActualDamage, DamageTypeCDO, EventInstigator, DamageCauser);
		OnTakeAnyDamage.Broadcast(this, ActualDamage, DamageTypeCDO, EventInstigator, DamageCauser);
		if (EventInstigator != nullptr)
		{
			EventInstigator->InstigatedAnyDamage(ActualDamage, DamageTypeCDO, this, DamageCauser);
		}
	}

	return ActualDamage;
}

float AActor::InternalTakeRadialDamage(float Damage, FRadialDamageEvent const& RadialDamageEvent, class AController* EventInstigator, class AActor* DamageCauser)
{
	float ActualDamage = Damage;

	FVector ClosestHitLoc(0);

	// find closest component
	// @todo, something more accurate here to account for size of components, e.g. closest point on the component bbox?
	// @todo, sum up damage contribution to each component?
	float ClosestHitDistSq = UE_MAX_FLT;
	for (int32 HitIdx=0; HitIdx<RadialDamageEvent.ComponentHits.Num(); ++HitIdx)
	{
		FHitResult const& Hit = RadialDamageEvent.ComponentHits[HitIdx];
		float const DistSq = (Hit.ImpactPoint - RadialDamageEvent.Origin).SizeSquared();
		if (DistSq < ClosestHitDistSq)
		{
			ClosestHitDistSq = DistSq;
			ClosestHitLoc = Hit.ImpactPoint;
		}
	}

	float const RadialDamageScale = RadialDamageEvent.Params.GetDamageScale( FMath::Sqrt(ClosestHitDistSq) );

	ActualDamage = FMath::Lerp(RadialDamageEvent.Params.MinimumDamage, ActualDamage, FMath::Max(0.f, RadialDamageScale));

	return ActualDamage;
}

float AActor::InternalTakePointDamage(float Damage, FPointDamageEvent const& PointDamageEvent, AController* EventInstigator, AActor* DamageCauser)
{
	return Damage;
}

/** Util to check if prim comp pointer is valid and still alive */
extern bool IsPrimCompValidAndAlive(UPrimitiveComponent* PrimComp);

/** Used to determine if it is ok to call a notification on this object */
bool IsActorValidToNotify(AActor* Actor)
{
	return IsValid(Actor) && !Actor->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists);
}

void AActor::InternalDispatchBlockingHit(UPrimitiveComponent* MyComp, UPrimitiveComponent* OtherComp, bool bSelfMoved, FHitResult const& Hit)
{
	check(MyComp);

	if (OtherComp != nullptr)
	{
		AActor* OtherActor = OtherComp->GetOwner();

		// Call virtual
		if(IsActorValidToNotify(this))
		{
			NotifyHit(MyComp, OtherActor, OtherComp, bSelfMoved, Hit.ImpactPoint, Hit.ImpactNormal, FVector(0,0,0), Hit);
		}

		// If we are still ok, call delegate on actor
		if(IsActorValidToNotify(this))
		{
			OnActorHit.Broadcast(this, OtherActor, FVector(0,0,0), Hit);
		}

		// If component is still alive, call delegate on component
		if(IsValidChecked(MyComp))
		{
			MyComp->OnComponentHit.Broadcast(MyComp, OtherActor, OtherComp, FVector(0,0,0), Hit);
		}
	}
}

void AActor::DispatchBlockingHit(UPrimitiveComponent* MyComp, UPrimitiveComponent* OtherComp, bool bSelfMoved, FHitResult const& Hit)
{
	InternalDispatchBlockingHit(MyComp, OtherComp, bSelfMoved, bSelfMoved ? Hit : FHitResult::GetReversedHit(Hit));
}


FString AActor::GetHumanReadableName() const
{
	return GetName();
}

void AActor::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	DisplayDebugManager.SetDrawColor(FColor(255, 0, 0));

	FString T = GetHumanReadableName();
	if( !IsValidChecked(this) )
	{
		T = FString::Printf(TEXT("%s DELETED (IsValid() == false)"), *T);
	}
	if( T != "" )
	{
		DisplayDebugManager.DrawString(T);
	}

	DisplayDebugManager.SetDrawColor(FColor(255, 255, 255));

	if( DebugDisplay.IsDisplayOn(TEXT("net")) )
	{
		if( GetNetMode() != NM_Standalone )
		{
			// networking attributes
			T = FString::Printf(TEXT("ROLE: %i RemoteRole: %i NetNode: %i"), (int32)GetLocalRole(), (int32)RemoteRole, (int32)GetNetMode());

			if( GetTearOff() )
			{
				T = T + FString(" Tear Off");
			}
			DisplayDebugManager.DrawString(T);
		}
	}

	DisplayDebugManager.DrawString(FString::Printf(TEXT("Location: %s Rotation: %s"), *GetActorLocation().ToCompactString(), *GetActorRotation().ToCompactString()));

	if( DebugDisplay.IsDisplayOn(TEXT("physics")) )
	{
		DisplayDebugManager.DrawString(FString::Printf(TEXT("Velocity: %s Speed: %f Speed2D: %f"), *GetVelocity().ToCompactString(), GetVelocity().Size(), GetVelocity().Size2D()));
	}

	if( DebugDisplay.IsDisplayOn(TEXT("collision")) )
	{
		Canvas->DrawColor.B = 0;
		float MyRadius, MyHeight;
		GetComponentsBoundingCylinder(MyRadius, MyHeight);
		DisplayDebugManager.DrawString(FString::Printf(TEXT("Collision Radius: %f Height: %f"), MyRadius, MyHeight));

		if ( RootComponent == nullptr )
		{
			DisplayDebugManager.DrawString(FString("No RootComponent"));
		}

		T = FString("Overlapping ");

		TSet<AActor*> TouchingActors;
		GetOverlappingActors(TouchingActors);
		bool bFoundAnyOverlaps = false;
		for (AActor* const TestActor : TouchingActors)
		{
			if (IsValid(TestActor))
			{
				T = T + TestActor->GetName() + " ";
				bFoundAnyOverlaps = true;
			}
		}

		if (!bFoundAnyOverlaps)
		{
			T = TEXT("Overlapping nothing");
		}
		DisplayDebugManager.DrawString(T);
	}
	DisplayDebugManager.DrawString(FString::Printf(TEXT(" Instigator: %s Owner: %s"), 
		*GetNameSafe(GetInstigator()), *GetNameSafe(Owner)));

	static FName NAME_Animation("Animation");
	static FName NAME_Bones = FName("Bones");
	if (DebugDisplay.IsDisplayOn(NAME_Animation) || DebugDisplay.IsDisplayOn(NAME_Bones))
	{
		if (DebugDisplay.IsDisplayOn(NAME_Animation))
		{
			for (UActorComponent* Comp : GetComponents())
			{
				if (USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(Comp))
				{
					if (UAnimInstance* AnimInstance = SkelMeshComp->GetAnimInstance())
					{
						AnimInstance->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
					}
				}
			}
		}
	}
}

void AActor::OutsideWorldBounds()
{
	Destroy();
}

bool AActor::CanBeBaseForCharacter(APawn* APawn) const
{
	return true;
}

void AActor::BecomeViewTarget( APlayerController* PC )
{
	K2_OnBecomeViewTarget(PC);
}

void AActor::EndViewTarget( APlayerController* PC )
{
	K2_OnEndViewTarget(PC);
}

APawn* AActor::GetInstigator() const
{
	return Instigator;
}

AController* AActor::GetInstigatorController() const
{
	return Instigator ? Instigator->GetController() : nullptr;
}

void AActor::CalcCamera(float DeltaTime, FMinimalViewInfo& OutResult)
{
	if (bFindCameraComponentWhenViewTarget)
	{
		// Look for the first active camera component and use that for the view
		TInlineComponentArray<UCameraComponent*> Cameras;
		GetComponents(/*out*/ Cameras);

		for (UCameraComponent* CameraComponent : Cameras)
		{
			if (CameraComponent->IsActive())
			{
				CameraComponent->GetCameraView(DeltaTime, OutResult);
				return;
			}
		}
	}

	GetActorEyesViewPoint(OutResult.Location, OutResult.Rotation);
}

bool AActor::HasActiveCameraComponent(bool bForceFindCamera) const
{
	if (bFindCameraComponentWhenViewTarget || bForceFindCamera)
	{
		// Look for the first active camera component and use that for the view
		for (const UActorComponent* Component : OwnedComponents)
		{
			const UCameraComponent* CameraComponent = Cast<const UCameraComponent>(Component);
			if (CameraComponent)
			{
				if (CameraComponent->IsActive())
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool AActor::HasActivePawnControlCameraComponent() const
{
	if (bFindCameraComponentWhenViewTarget)
	{
		// Look for the first active camera component and use that for the view
		for (const UActorComponent* Component : OwnedComponents)
		{
			const UCameraComponent* CameraComponent = Cast<const UCameraComponent>(Component);
			if (CameraComponent)
			{
				if (CameraComponent->IsActive() && CameraComponent->bUsePawnControlRotation)
				{
					return true;
				}
			}
		}
	}
	return false;
}

void AActor::ForceNetRelevant()
{
	if ( !NeedsLoadForClient() )
	{
		UE_LOGF(LogSpawn, Warning, "ForceNetRelevant called for actor that doesn't load on client: %ls", *GetFullName() );
		return;
	}

	if (RemoteRole == ROLE_None)
	{
		SetReplicates(true);
		bAlwaysRelevant = true;
		if (GetNetUpdateFrequency() == 0.f)
		{
			SetNetUpdateFrequency(0.1f);
		}
	}
	ForceNetUpdate();
}

void AActor::GetActorEyesViewPoint( FVector& OutLocation, FRotator& OutRotation ) const
{
	OutLocation = GetActorLocation();
	OutRotation = GetActorRotation();
}

enum ECollisionResponse AActor::GetComponentsCollisionResponseToChannel(enum ECollisionChannel Channel) const
{
	ECollisionResponse OutResponse = ECR_Ignore;

	for (UActorComponent* ActorComponent : OwnedComponents)
	{
		UPrimitiveComponent* Primitive = Cast<UPrimitiveComponent>(ActorComponent);
		if ( Primitive && Primitive->IsCollisionEnabled() )
		{
			// find Max of the response, blocking > overlapping > ignore
			OutResponse = FMath::Max(Primitive->GetCollisionResponseToChannel(Channel), OutResponse);
		}
	}

	return OutResponse;
};

void AActor::AddOwnedComponent(UActorComponent* Component)
{
	check(Component->GetOwner() == this);

	// Note: we do not mark dirty here because this can be called when in editor when modifying transient components
	// if a component is added during this time it should not dirty.  Higher level code in the editor should always dirty the package anyway
	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	bool bAlreadyInSet = false;
	OwnedComponents.Add(Component, &bAlreadyInSet);

	if (!bAlreadyInSet)
	{
		if (Component->GetIsReplicated())
		{
			ReplicatedComponents.AddUnique(Component);

			AddComponentForReplication(Component);
		}

		if (Component->IsCreatedByConstructionScript())
		{
			BlueprintCreatedComponents.Add(Component);
		}
		else if (Component->CreationMethod == EComponentCreationMethod::Instance)
		{
			InstanceComponents.Add(Component);
		}
	}
}

void AActor::RemoveOwnedComponent(UActorComponent* Component)
{
	// Note: we do not mark dirty here because this can be called as part of component duplication when reinstancing components during blueprint compilation
	// if a component is removed during this time it should not dirty.  Higher level code in the editor should always dirty the package anyway.
	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	if (OwnedComponents.Remove(Component) > 0)
	{
		ReplicatedComponents.RemoveSingleSwap(Component);
		RemoveReplicatedComponent(Component);

		if (Component->IsCreatedByConstructionScript())
		{
			BlueprintCreatedComponents.RemoveSingleSwap(Component);
		}
		else if (Component->CreationMethod == EComponentCreationMethod::Instance)
		{
			InstanceComponents.RemoveSingleSwap(Component);
		}

		
	}
}

#if DO_CHECK || USING_CODE_ANALYSIS
bool AActor::OwnsComponent(UActorComponent* Component) const
{
	return OwnedComponents.Contains(Component);
}
#endif

void AActor::UpdateReplicatedComponent(UActorComponent* Component)
{
	using namespace UE::Net;
	
	bool bWasRemovedFromReplicatedComponents = false;

	if (Component->GetIsReplicated())
	{
		ReplicatedComponents.AddUnique(Component);
	}
	else
	{
		bWasRemovedFromReplicatedComponents = ReplicatedComponents.RemoveSingleSwap(Component) > 0;
	}


	const ELifetimeCondition NetCondition = Component->GetIsReplicated() ? AllowActorComponentToReplicate(Component) : COND_Never;
	
	if (Component->GetIsReplicated() && ActorHasBegunPlay != EActorBeginPlayState::HasNotBegunPlay && !Component->IsReadyForReplication())
	{
		Component->ReadyForReplication();
	}

	if (FReplicatedComponentInfo* RepComponentInfo = ReplicatedComponentsInfo.FindByKey(Component))
	{
		// Even if not replicated anymore just set the condition to Never because we want to keep the subobject list intact in case the component switches back to replicated later.
		RepComponentInfo->NetCondition = NetCondition;
	}
	// Keep track of subobjects for replicated components even if Allow returns Never. The condition might get changed later on and we want to start tracking subobjects immediately.
	else if (Component->GetIsReplicated())
	{
		ReplicatedComponentsInfo.Emplace(FReplicatedComponentInfo(Component, NetCondition));
	}

	if (HasAuthority())
	{
		if (NetCondition != COND_Never && HasActorBegunPlay())
		{
			// Start replication and set NetCondition, if component already is replicated the NetCondition will be updated
			UE::Net::FReplicationSystemUtil::StartReplicatingActorComponent(this, Component);
		}
		else if (bWasRemovedFromReplicatedComponents || NetCondition == COND_Never)
		{
			if (GbCallStopReplicationForActorComponentWhenUpdatingReplicatedComponent)
			{
				UE::Net::FReplicationSystemUtil::StopReplicatingActorComponent(Component);
			}
			else
			{
				UE::Net::FReplicationSystemUtil::SetNetConditionForActorComponent(Component, COND_Never);
			}
		}
	}
}

void AActor::UpdateAllReplicatedComponents()
{
	using namespace UE::Net;

	ReplicatedComponents.Reset();

	// Disable all replicated components first
	for (FReplicatedComponentInfo& RepComponentInfo : ReplicatedComponentsInfo)
	{
		RepComponentInfo.NetCondition = COND_Never;
		if (HasAuthority())
		{
			// Need to stop replication for all components that should no longer replicate
			if (RepComponentInfo.Component && !RepComponentInfo.Component->GetIsReplicated())
			{
				if (GbCallStopReplicationForActorComponentWhenUpdatingReplicatedComponent)
				{
					UE::Net::FReplicationSystemUtil::StopReplicatingActorComponent(RepComponentInfo.Component);
				}
				else
				{
					UE::Net::FReplicationSystemUtil::SetNetConditionForActorComponent(RepComponentInfo.Component, COND_Never);
				}
			}
		}
	}

	for (UActorComponent* Component : OwnedComponents)
	{
		if (Component && Component->GetIsReplicated())
		{
			// We reset the array so no need to add unique
			ReplicatedComponents.Add(Component);

			if (ActorHasBegunPlay != EActorBeginPlayState::HasNotBegunPlay)
			{
				const ELifetimeCondition NetCondition = AllowActorComponentToReplicate(Component);
				const int32 Index = ReplicatedComponentsInfo.AddUnique(FReplicatedComponentInfo(Component));
				ReplicatedComponentsInfo[Index].NetCondition = NetCondition;

				if (!Component->IsReadyForReplication())
				{
					Component->ReadyForReplication();
				}

				if (HasAuthority())
				{
					if (NetCondition != COND_Never)
					{
						// Start replication and set NetCondition, if component already is replicated the NetCondition will be updated
						UE::Net::FReplicationSystemUtil::StartReplicatingActorComponent(this, Component);
					}
					else
					{
						if (GbCallStopReplicationForActorComponentWhenUpdatingReplicatedComponent)
						{
							UE::Net::FReplicationSystemUtil::StopReplicatingActorComponent(Component);
						}
						else
						{
							UE::Net::FReplicationSystemUtil::SetNetConditionForActorComponent(Component, COND_Never);
						}
					}
				}
			}
		}
	}
}

const TArray<UActorComponent*>& AActor::GetInstanceComponents() const
{
	return InstanceComponents;
}

void AActor::AddInstanceComponent(UActorComponent* Component)
{
	Component->CreationMethod = EComponentCreationMethod::Instance;
	InstanceComponents.AddUnique(Component);
}

void AActor::RemoveInstanceComponent(UActorComponent* Component)
{
	InstanceComponents.Remove(Component);
}

void AActor::ClearInstanceComponents(const bool bDestroyComponents)
{
	if (bDestroyComponents)
	{
		// Need to cache because calling destroy will remove them from InstanceComponents
		TArray<UActorComponent*> CachedComponents(InstanceComponents);

		// Run in reverse to reduce memory churn when the components are removed from InstanceComponents
		for (UActorComponent* CachedComponent : ReverseIterate(CachedComponents))
		{
			if (CachedComponent)
			{
				CachedComponent->DestroyComponent();
			}
		}
	}
	else
	{
		InstanceComponents.Reset();
	}
}

UActorComponent* AActor::FindComponentByClass(const TSubclassOf<UActorComponent> ComponentClass) const
{
	UActorComponent* FoundComponent = nullptr;

	if (UClass* TargetClass = ComponentClass.Get())
	{
		for (UActorComponent* Component : OwnedComponents)
		{
			if (Component && Component->IsA(TargetClass))
			{
				FoundComponent = Component;
				break;
			}
		}
	}

	return FoundComponent;
}

UActorComponent* AActor::GetComponentByClass(TSubclassOf<UActorComponent> ComponentClass) const
{
	return FindComponentByClass(ComponentClass);
}

TArray<UActorComponent*> AActor::K2_GetComponentsByClass(TSubclassOf<UActorComponent> ComponentClass) const
{
	TArray<UActorComponent*> Components;
	GetComponents(ComponentClass, Components);
	return MoveTemp(Components);
}

UActorComponent* AActor::FindComponentByTag(TSubclassOf<UActorComponent> ComponentClass, FName Tag) const
{
	for (UActorComponent* Component : GetComponents())
	{
		if (Component && Component->IsA(ComponentClass) && Component->ComponentHasTag(Tag))
		{
			return Component;
		}
	}

	return nullptr;
}

TArray<UActorComponent*> AActor::GetComponentsByTag(TSubclassOf<UActorComponent> ComponentClass, FName Tag) const
{
	TInlineComponentArray<UActorComponent*> ComponentsByClass;
	GetComponents(ComponentClass, ComponentsByClass);

	TArray<UActorComponent*> ComponentsByTag;
	ComponentsByTag.Reserve(ComponentsByClass.Num());
	for (UActorComponent* Component : ComponentsByClass)
	{
		if (Component->ComponentHasTag(Tag))
		{
			ComponentsByTag.Push(Component);
		}
	}

	return MoveTemp(ComponentsByTag);
}

UActorComponent* AActor::FindComponentByInterface(const TSubclassOf<UInterface> Interface) const
{
	UActorComponent* FoundComponent = nullptr;

	if (Interface)
	{
		for (UActorComponent* Component : GetComponents())
		{
			if (Component && Component->GetClass()->ImplementsInterface(Interface))
			{
				FoundComponent = Component;
				break;
			}
		}
	}

	return FoundComponent;
}

TArray<UActorComponent*> AActor::GetComponentsByInterface(TSubclassOf<UInterface> Interface) const
{
	TArray<UActorComponent*> Components;

	if (Interface)
	{
		for (UActorComponent* Component : GetComponents())
		{
			if (Component && Component->GetClass()->ImplementsInterface(Interface))
			{
				Components.Add(Component);
			}
		}
	}

	return Components;
}

void AActor::DisableComponentsSimulatePhysics()
{
	for (UActorComponent* Component : GetComponents())
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
		{
			PrimComp->SetSimulatePhysics(false);
		}
	}
}

void AActor::PreRegisterAllComponents()
{
	// Here we don't ensure that bHasPreRegisteredAllComponents == false because there are cases where PreRegisterAllComponents() can be called multiple times (SpawnActor is one example)
	bHasPreRegisteredAllComponents = true;
}

void AActor::PostRegisterAllComponents() 
{
	ensureMsgf(bHasRegisteredAllComponents == true, TEXT("bHasRegisteredAllComponents must be set to true prior to calling PostRegisterAllComponents()"));

	FNavigationSystem::OnActorRegistered(*this);

	FActorInstanceGuid::ReleaseActorInstanceGuid(*this);
}

/** Util to call OnComponentCreated on components */
static void DispatchOnComponentsCreated(AActor* NewActor)
{
	TInlineComponentArray<UActorComponent*> Components;
	NewActor->GetComponents(Components);

	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(NewActor->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(NewActor->GetClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET(NewActor, NewActor->GetClass());
	for (UActorComponent* ActorComp : Components)
	{
		if (!ActorComp->HasBeenCreated())
		{
			ActorComp->OnComponentCreated();
		}
	}
}

#if WITH_EDITOR

bool AActor::CanDeleteSelectedActor(FText& OutReason) const
{
	return ActorUtils::CanDeleteOrReplaceCommon(this, OutReason);
}

bool AActor::CanReplaceSelectedActor(FText& OutReason) const
{
	return ActorUtils::CanDeleteOrReplaceCommon(this, OutReason);
}

void AActor::PostEditImport()
{
	Super::PostEditImport();

	FixupDataLayers();

	DispatchOnComponentsCreated(this);

	ULevel* Level = GetLevel();
	if (Level && Level->IsUsingActorFolders())
	{
		if ((GetFolderPath().IsNone() || !FolderGuid.IsValid()) && !FolderPath.IsNone())
		{
			SetFolderPath(FolderPath);
		}
	}
}

bool AActor::IsSelectedInEditor() const
{
	return IsValidChecked(this) && GIsActorSelectedInEditor && GIsActorSelectedInEditor(this);
}

bool AActor::SupportsExternalPackaging() const
{
	if (HasAllFlags(RF_Transient) || GetClass()->HasAllClassFlags(CLASS_Transient))
	{
		return false;
	}

	if (!IsValidChecked(this))
	{
		return false;
	}

	if (!IsTemplate())
	{
		if (IsChildActor())
		{
			return false;
		}

		if (FFoliageHelper::IsOwnedByFoliage(this))
		{
			return false;
		}
	}

	return true;
}
#endif

/** Util that sets up the actor's component hierarchy (when users forget to do so, in their native ctor) */
static USceneComponent* FixupNativeActorComponents(AActor* Actor)
{
	USceneComponent* SceneRootComponent = Actor->GetRootComponent();
	if (SceneRootComponent == nullptr)
	{
		TInlineComponentArray<USceneComponent*> SceneComponents;
		Actor->GetComponents(SceneComponents);
		if (SceneComponents.Num() > 0)
		{
			UE_LOGF(LogActor, Warning, "%ls has natively added scene component(s), but none of them were set as the actor's RootComponent - picking one arbitrarily", *Actor->GetFullName());
	
			// if the user forgot to set one of their native components as the root, 
			// we arbitrarily pick one for them (otherwise the SCS could attempt to 
			// create its own root, and nest native components under it)
			for (USceneComponent* Component : SceneComponents)
			{
				if ((Component == nullptr) ||
					(Component->GetAttachParent() != nullptr) ||
					(Component->CreationMethod != EComponentCreationMethod::Native))
				{
					continue;
				}

				SceneRootComponent = Component;
				Actor->SetRootComponent(Component);
				break;
			}
		}
	}

	return SceneRootComponent;
}

// Simple and short-lived cache for storing transforms between beginning and finishing spawning.
static TMap< TWeakObjectPtr<AActor>, FTransform > GSpawnActorDeferredTransformCache;

static void ValidateDeferredTransformCache()
{
	// clean out any entries where the actor is no longer valid
	// could happen if an actor is destroyed before FinishSpawning is called
	for (auto It = GSpawnActorDeferredTransformCache.CreateIterator(); It; ++It)
	{
		const TWeakObjectPtr<AActor>& ActorRef = It.Key();
		if (ActorRef.IsValid() == false)
		{
			It.RemoveCurrent();
		}
	}
}

void AActor::PostSpawnInitialize(FTransform const& UserSpawnTransform, AActor* InOwner, APawn* InInstigator, bool bRemoteOwned, bool bNoFail, bool bDeferConstruction, ESpawnActorScaleMethod TransformScaleMethod)
{
	// General flow here is like so
	// - Actor sets up the basics.
	// - Actor gets PreInitializeComponents()
	// - Actor constructs itself, after which its components should be fully assembled
	// - Actor components get OnComponentCreated
	// - Actor components get InitializeComponent
	// - Actor gets PostInitializeComponents() once everything is set up
	//
	// This should be the same sequence for deferred or nondeferred spawning.

	// It's not safe to call UWorld accessor functions till the world info has been spawned.
	UWorld* const World = GetWorld();
	bool const bActorsInitialized = World && World->AreActorsInitialized();

	CreationTime = (World ? World->GetTimeSeconds() : 0.f);

	// Set network role.
	ensureMsgf(GetLocalRole() == ROLE_Authority, TEXT("Actor %s has an invalid Role and may be a corrupt asset!"), *GetFullName());
	ExchangeNetRoles(bRemoteOwned);

	// Set owner.
	SetOwner(InOwner);

	// Set instigator
	SetInstigator(InInstigator);

	// Set the actor's world transform if it has a native rootcomponent.
	USceneComponent* const SceneRootComponent = FixupNativeActorComponents(this);
	if (SceneRootComponent != nullptr)
	{
		check(SceneRootComponent->GetOwner() == this);

		// Respect any non-default transform value that the root component may have received from the archetype that's owned
		// by the native CDO, so the final transform might not always necessarily equate to the passed-in UserSpawnTransform.
		const FTransform RootTransform(SceneRootComponent->GetRelativeRotation(), SceneRootComponent->GetRelativeLocation(), SceneRootComponent->GetRelativeScale3D());
		FTransform FinalRootComponentTransform = RootTransform;
		switch(TransformScaleMethod)
		{
		case ESpawnActorScaleMethod::OverrideRootScale:
			FinalRootComponentTransform = UserSpawnTransform;
			break;
		case ESpawnActorScaleMethod::MultiplyWithRoot:
		case ESpawnActorScaleMethod::SelectDefaultAtRuntime:
			FinalRootComponentTransform = RootTransform * UserSpawnTransform;
			break;
		}
		SceneRootComponent->SetWorldTransform(FinalRootComponentTransform, false, nullptr, ETeleportType::ResetPhysics);
	}

	// Call OnComponentCreated on all default (native) components
	DispatchOnComponentsCreated(this);

	// Register the actor's default (native) components, but only if we have a native scene root. If we don't, it implies that there could be only non-scene components
	// at the native class level. In that case, if this is a Blueprint instance, we need to defer native registration until after SCS execution can establish a scene root.
	// Note: This API will also call PostRegisterAllComponents() on the actor instance. If deferred, PostRegisterAllComponents() won't be called until the root is set by SCS.
	bHasDeferredComponentRegistration = (SceneRootComponent == nullptr && Cast<UBlueprintGeneratedClass>(GetClass()) != nullptr);
	if (!bHasDeferredComponentRegistration && GetWorld())
	{
		RegisterAllComponents();
	}

#if WITH_EDITOR
	// When placing actors in the editor, init any random streams 
	if (!bActorsInitialized)
	{
		SeedAllRandomStreams();
	}
#endif

	// See if anything has deleted us
	if( !IsValidChecked(this) && !bNoFail )
	{
		return;
	}

	// Send messages. We've fully spawned
	PostActorCreated();

	// Executes native and BP construction scripts.
	// After this, we can assume all components are created and assembled.
	if (!bDeferConstruction)
	{
		FinishSpawning(UserSpawnTransform, true);
	}
	else if (SceneRootComponent != nullptr)
	{
		// we have a native root component and are deferring construction, store our original UserSpawnTransform
		// so we can do the proper thing if the user passes in a different transform during FinishSpawning
		GSpawnActorDeferredTransformCache.Emplace(this, UserSpawnTransform);
	}
}

#include "GameFramework/SpawnActorTimer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Actor)

void AActor::FinishSpawning(const FTransform& UserTransform, bool bIsDefaultTransform, const FComponentInstanceDataCache* InstanceDataCache, ESpawnActorScaleMethod TransformScaleMethod)
{
#if ENABLE_SPAWNACTORTIMER
	FScopedSpawnActorTimer SpawnTimer(GetClass()->GetFName(), ESpawnActorTimingType::FinishSpawning);
	SpawnTimer.SetActorName(GetFName());
#endif

	if (ensure(!bHasFinishedSpawning))
	{
		bHasFinishedSpawning = true;

		FTransform FinalRootComponentTransform = (RootComponent ? RootComponent->GetComponentTransform() : UserTransform);

		// see if we need to adjust the transform (i.e. in deferred cases where the caller passes in a different transform here 
		// than was passed in during the original SpawnActor call)
		if (RootComponent && !bIsDefaultTransform)
		{
			FTransform OriginalSpawnTransform(NoInit);
			const bool bFound = GSpawnActorDeferredTransformCache.RemoveAndCopyValue(this, OriginalSpawnTransform);

			if (bFound && !OriginalSpawnTransform.Equals(UserTransform))
			{
				UserTransform.GetLocation().DiagnosticCheckNaN(TEXT("AActor::FinishSpawning: UserTransform.GetLocation()"));
				UserTransform.GetRotation().DiagnosticCheckNaN(TEXT("AActor::FinishSpawning: UserTransform.GetRotation()"));

				// caller passed a different transform!
				// undo the original spawn transform to get back to the template transform, so we can recompute a good
				// final transform that takes into account the template's transform
				FTransform const TemplateTransform = RootComponent->GetComponentTransform() * OriginalSpawnTransform.Inverse();
				FinalRootComponentTransform = TemplateTransform * UserTransform;
			}

			// should be fast and relatively rare
			ValidateDeferredTransformCache();
		}

		FinalRootComponentTransform.GetLocation().DiagnosticCheckNaN(TEXT("AActor::FinishSpawning: FinalRootComponentTransform.GetLocation()"));
		FinalRootComponentTransform.GetRotation().DiagnosticCheckNaN(TEXT("AActor::FinishSpawning: FinalRootComponentTransform.GetRotation()"));

		{
			FEditorScriptExecutionGuard ScriptGuard;
			ExecuteConstruction(FinalRootComponentTransform, nullptr, InstanceDataCache, bIsDefaultTransform, TransformScaleMethod);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_PostActorConstruction);
			PostActorConstruction();
		}

		if (UWorld* World = GetWorld())
		{
			World->OnActorFinishedSpawning(this);
		}
	}
}

void AActor::PostActorConstruction()
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET(this, GetClass());
	UWorld* const World = GetWorld();
	bool const bActorsInitialized = World && World->AreActorsInitialized();

	if (bActorsInitialized)
	{
		PreInitializeComponents();
	}

	// If this is dynamically spawned replicated actor, defer calls to BeginPlay and UpdateOverlaps until replicated properties are deserialized
	const bool bDeferBeginPlayAndUpdateOverlaps = (bExchangedRoles && RemoteRole == ROLE_Authority) && !GIsReinstancing;

	if (bActorsInitialized)
	{
		// Call InitializeComponent on components
		InitializeComponents();

		// actor should have all of its components created and registered now, do any collision checking and handling that we need to do
		if (World)
		{
			switch (SpawnCollisionHandlingMethod)
			{
			case ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn:
			{
				// Try to find a spawn position
				FVector AdjustedLocation = GetActorLocation();
				FRotator AdjustedRotation = GetActorRotation();
				if (World->FindTeleportSpot(this, AdjustedLocation, AdjustedRotation))
				{
					SetActorLocationAndRotation(AdjustedLocation, AdjustedRotation, false, nullptr, ETeleportType::TeleportPhysics);
				}
			}
			break;
			case ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding:
			{
				// Try to find a spawn position			
				FVector AdjustedLocation = GetActorLocation();
				FRotator AdjustedRotation = GetActorRotation();
				if (World->FindTeleportSpot(this, AdjustedLocation, AdjustedRotation))
				{
					SetActorLocationAndRotation(AdjustedLocation, AdjustedRotation, false, nullptr, ETeleportType::TeleportPhysics);
				}
				else
				{
					UE_LOGF(LogSpawn, Warning, "SpawnActor failed because of collision at the spawn location [%ls] for [%ls]", *AdjustedLocation.ToString(), *GetClass()->GetName());
					Destroy();
				}
			}
			break;
			case ESpawnActorCollisionHandlingMethod::DontSpawnIfColliding:
				if (World->EncroachingBlockingGeometry(this, GetActorLocation(), GetActorRotation()))
				{
					UE_LOGF(LogSpawn, Warning, "SpawnActor failed because of collision at the spawn location [%ls] for [%ls]", *GetActorLocation().ToString(), *GetClass()->GetName());
					Destroy();
				}
				break;
			case ESpawnActorCollisionHandlingMethod::Undefined:
			case ESpawnActorCollisionHandlingMethod::AlwaysSpawn:
			default:
				// note we use "always spawn" as default, so treat undefined as that
				// nothing to do here, just proceed as normal
				break;
			}
		}

		if (IsValidChecked(this))
		{
			PostInitializeComponents();
			if (IsValidChecked(this))
			{
				if (!bActorInitialized)
				{
					UE_LOGF(LogActor, Fatal, "%ls failed to route PostInitializeComponents.  Please call Super::PostInitializeComponents() in your <className>::PostInitializeComponents() function. ", *GetFullName());
				}

				bool bRunBeginPlay = !bDeferBeginPlayAndUpdateOverlaps && (BeginPlayCallDepth > 0 || World->HasBegunPlay());
				if (bRunBeginPlay)
				{
					if (AActor* ParentActor = GetParentActor())
					{
						// Child Actors cannot run begin play until their parent has run
						bRunBeginPlay = (ParentActor->HasActorBegunPlay() || ParentActor->IsActorBeginningPlay());
					}
				}

#if WITH_EDITOR
				if (bRunBeginPlay && bIsEditorPreviewActor)
				{
					bRunBeginPlay = false;
				}
#endif

				if (bRunBeginPlay)
				{
					SCOPE_CYCLE_COUNTER(STAT_ActorBeginPlay);
					DispatchBeginPlay();
				}
			}
		}
	}
	else
	{
		// Invalidate the object so that when the initial undo record is made,
		// the actor will be treated as destroyed, in that undo an add will
		// actually work
		MarkAsGarbage();
		Modify(false);
		ClearGarbage();
	}
}

void AActor::SetReplicates(bool bInReplicates)
{ 
	// Due to SetRemoteRoleForBackwardsCompat, it's possible that bReplicates is false, but RemoteRole is something other than ROLE_None.
	// So, we'll also make sure that we don't need to update RemoteRole here, even if not bReplicates wouldn't change, to fix up that case.
	const ENetRole ExpectedRemoteRole = bInReplicates ? ROLE_SimulatedProxy : ROLE_None;

	if (GetLocalRole() != ROLE_Authority)
	{
		UE_LOGF(LogActor, Warning, "SetReplicates called on actor '%ls' that is not valid for having its role modified.", *GetName());
	}
	else if (bReplicates != bInReplicates || RemoteRole != ExpectedRemoteRole) 
	{
		const bool bNewlyReplicates = (bReplicates == false && bInReplicates == true);
	
		RemoteRole = ExpectedRemoteRole;
		bReplicates = bInReplicates;

		// Initialized actors will immediately get added/removed from the ReplicationSystem
		// Uninitialized actors should set bReplicates instead as SetReplicates() will change the RemoteRole earlier then normal
		if (bActorInitialized)
		{
			if (bReplicates)
			{
				// GetWorld will return nullptr on CDO, FYI
				if (UWorld* MyWorld = GetWorld())
				{
					// Only call into net driver if we just started replicating changed
					// This actor should already be in the Network Actors List if it was already replicating.
					if (bNewlyReplicates)
					{
						MyWorld->AddNetworkActor(this);
					}

					if (HasActorBegunPlay())
					{
						if (bNewlyReplicates)
						{
							UE::Net::FReplicationSystemUtil::StartReplicatingActor(this);
						}
						else
						{
							UpdateOwningNetConnection();
						}
					}

					ForcePropertyCompare();
				}
			}
			else if (HasActorBegunPlay())
			{
				UE::Net::FReplicationSystemUtil::StopReplicatingActor(this, FStopReplicatingActorParams(EEndPlayReason::RemovedFromWorld));
			}

			MARK_PROPERTY_DIRTY_FROM_NAME(AActor, RemoteRole, this);
		}
	}
}

void AActor::SetReplicateMovement(bool bInReplicateMovement)
{
	const bool bWasReplicatingMovement = IsReplicatingMovement();
	SetReplicatingMovement(bInReplicateMovement);

	// Update per-frame physics quantization registration
	if (bWasReplicatingMovement != bInReplicateMovement)
	{
		if (UWorld* World = GetWorld())
		{
			if (FPhysScene_Chaos* Scene = static_cast<FPhysScene_Chaos*>(World->GetPhysicsScene()))
			{
				if (FPhysicsReplicationQuantization* Quant = Scene->GetPhysicsReplicationQuantization())
				{
					if (bInReplicateMovement)
					{
						Quant->RegisterActor(this);
					}
					else
					{
						Quant->UnregisterActor(this);
					}
				}
			}
		}
	}
}

void AActor::SetAutonomousProxy(const bool bInAutonomousProxy, const bool bAllowForcePropertyCompare)
{
	if (bReplicates)
	{
		const TEnumAsByte<enum ENetRole> OldRemoteRole = RemoteRole;

		RemoteRole = (bInAutonomousProxy ? ROLE_AutonomousProxy : ROLE_SimulatedProxy);

		if (bAllowForcePropertyCompare && RemoteRole != OldRemoteRole)
		{
			// We intentionally only mark RemoteRole dirty after this check, because Networking Code
			// for swapping roles will call SetAutonomousProxy to downgrade RemoteRole.
			// However, that's the only code that should pass bAllowForcePropertyCompare=false.
			// Everywhere else should use bAllowForcePropertyCompare=true.
			// TODO: Maybe split these methods up to make it clearer.
			MARK_PROPERTY_DIRTY_FROM_NAME(AActor, RemoteRole, this);

			// We have to do this so the role change above will replicate (turn off shadow state sharing for a frame)
			// This is because RemoteRole is special since it will change between connections, so we have to special case
			ForcePropertyCompare();

			// Update autonomous role condition
			if (UReplicationSystem* ReplicationSystem = UE::Net::FReplicationSystemUtil::GetReplicationSystem(this))
			{
				if (const UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
				{
					const UE::Net::FNetRefHandle RefHandle = Bridge->GetReplicatedRefHandle(this);
					if (RefHandle.IsValid())
					{
						const uint32 OwningNetConnectionId = ReplicationSystem->GetOwningNetConnection(RefHandle);
						const bool bEnableAutonomousCondition = RemoteRole == ROLE_AutonomousProxy;
						ReplicationSystem->SetReplicationConditionConnectionFilter(RefHandle, UE::Net::EReplicationCondition::RoleAutonomous, OwningNetConnectionId, bEnableAutonomousCondition);
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogActor, Warning, "SetAutonomousProxy called on a unreplicated actor '%ls", *GetName());
	}
}

void AActor::CopyRemoteRoleFrom(const AActor* CopyFromActor)
{
	RemoteRole = CopyFromActor->GetRemoteRole();
	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, RemoteRole, this);

	if (RemoteRole != ROLE_None)
	{
		GetWorld()->AddNetworkActor(this);
		ForcePropertyCompare();
	}
}

void AActor::PostNetInit()
{
	if(RemoteRole != ROLE_Authority)
	{
		UE_LOGF(LogActor, Warning, "AActor::PostNetInit %ls Remoterole: %d", *GetName(), (int)RemoteRole);
	}

	if (!HasActorBegunPlay())
	{
		const UWorld* MyWorld = GetWorld();
		if (MyWorld && MyWorld->HasBegunPlay())
		{
			SCOPE_CYCLE_COUNTER(STAT_ActorBeginPlay);
			DispatchBeginPlay();
		}
	}
}

void AActor::ExchangeNetRoles(bool bRemoteOwned)
{
	checkf(!HasAnyFlags(RF_ClassDefaultObject), TEXT("ExchangeNetRoles should never be called on a CDO as it causes issues when replicating actors over the network due to mutated transient data!"));

	if (!bExchangedRoles)
	{
		if (bRemoteOwned)
		{
			Exchange( Role, RemoteRole );
		}
		bExchangedRoles = true;
	}
}

void AActor::SwapRoles()
{
	Swap(Role, RemoteRole);

	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, RemoteRole, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, Role, this);

	ForcePropertyCompare();
}

EActorUpdateOverlapsMethod AActor::GetUpdateOverlapsMethodDuringLevelStreaming() const
{
	if (UpdateOverlapsMethodDuringLevelStreaming == EActorUpdateOverlapsMethod::UseConfigDefault)
	{
		// In the case of a default value saying "use defaults", pick something else.
		return (DefaultUpdateOverlapsMethodDuringLevelStreaming != EActorUpdateOverlapsMethod::UseConfigDefault) ? DefaultUpdateOverlapsMethodDuringLevelStreaming : EActorUpdateOverlapsMethod::AlwaysUpdate;
	}
	return UpdateOverlapsMethodDuringLevelStreaming;
}

void AActor::DispatchBeginPlay(bool bFromLevelStreaming)
{
	// If we are spawned from networking, the actor is not ready for begin play until the initial state has been applied.
	if (bActorIsPendingPostNetInit)
	{
		if (UE::Net::FReplicationSystemUtil::GetReplicationSystem(this))
		{
			return;
		}
	}
	
	UWorld* World = (!HasActorBegunPlay() && IsValidChecked(this) ? GetWorld() : nullptr);

	if (World)
	{
		SCOPE_CYCLE_UOBJECT(BeginPlay, this);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetClass(), ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET(this, GetClass());

		ensureMsgf(ActorHasBegunPlay == EActorBeginPlayState::HasNotBegunPlay, TEXT("BeginPlay was called on actor %s which was in state %d"), *GetPathName(), (int32)ActorHasBegunPlay);
		const uint32 CurrentCallDepth = BeginPlayCallDepth++;

		bActorBeginningPlayFromLevelStreaming = bFromLevelStreaming;

		// Call this before updating status of ActorHasBegunPlay
		BuildReplicatedComponentsInfo();

		ActorHasBegunPlay = EActorBeginPlayState::BeginningPlay;

		UE::Net::FReplicationSystemUtil::StartReplicatingActor(this);

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
		// If we're loading from level streaming, allow a BeginPlay since it may spawn additional Actors
		if (bFromLevelStreaming || World->IsActorTickAndUserCallbacksEnabled())
#endif
		{
			BeginPlay();
		}
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
		else
		{
			ActorHasBegunPlay = EActorBeginPlayState::HasBegunPlay;
		}
#endif

		ensure(BeginPlayCallDepth - 1 == CurrentCallDepth);
		BeginPlayCallDepth = CurrentCallDepth;

		if (bActorWantsDestroyDuringBeginPlay)
		{
			// Pass true for bNetForce as either it doesn't matter or it was true the first time to even 
			// get to the point we set bActorWantsDestroyDuringBeginPlay to true
			World->DestroyActor(this, true); 
		}
		
		if (IsValidChecked(this)
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
			&& World->IsActorTickAndUserCallbacksEnabled()
#endif
		)
		{
			// Initialize overlap state
			UpdateInitialOverlaps(bFromLevelStreaming);
		}

		bActorBeginningPlayFromLevelStreaming = false;
	}
}

void AActor::BeginPlay()
{
	TRACE_OBJECT_LIFETIME_BEGIN(this);

	ensureMsgf(ActorHasBegunPlay == EActorBeginPlayState::BeginningPlay, TEXT("BeginPlay was called on actor %s which was in state %d"), *GetPathName(), (int32)ActorHasBegunPlay);
	SetLifeSpan( InitialLifeSpan );
	RegisterAllActorTickFunctions(true, false); // Components are done below.

	TInlineComponentArray<UActorComponent*> Components;
	GetComponents(Components);

	for (UActorComponent* Component : Components)
	{
		// bHasBegunPlay will be true for the component if the component was renamed and moved to a new outer during initialization
		if (Component->IsRegistered() && !Component->HasBegunPlay())
		{
			Component->RegisterAllComponentTickFunctions(true);
			Component->BeginPlay();
			ensureMsgf(Component->HasBegunPlay(), TEXT("Failed to route BeginPlay (%s)"), *Component->GetFullName());
		}
		else
		{
			// When an Actor begins play we expect only the not bAutoRegister false components to not be registered
			//check(!Component->bAutoRegister);
		}
	}

	if (GetAutoDestroyWhenFinished())
	{
		if (UWorld* MyWorld = GetWorld())
		{
			if (UAutoDestroySubsystem* AutoDestroySys = MyWorld->GetSubsystem<UAutoDestroySubsystem>())
			{
				AutoDestroySys->RegisterActor(this);
			}			
		}
	}

	ReceiveBeginPlay();

	ActorHasBegunPlay = EActorBeginPlayState::HasBegunPlay;
}

void AActor::UpdateInitialOverlaps(bool bFromLevelStreaming)
{
	if (!bFromLevelStreaming)
	{
		UpdateOverlaps();
	}
	else
	{
		// Note: Conditionally doing notifies here since loading or streaming in isn't actually conceptually beginning a touch.
		//	     Rather, it was always touching and the mechanics of loading is just an implementation detail.
		if (bGenerateOverlapEventsDuringLevelStreaming)
		{
			UpdateOverlaps(bGenerateOverlapEventsDuringLevelStreaming);
		}
		else
		{
			bool bUpdateOverlaps = true;
			const EActorUpdateOverlapsMethod UpdateMethod = GetUpdateOverlapsMethodDuringLevelStreaming();
			switch (UpdateMethod)
			{
				case EActorUpdateOverlapsMethod::OnlyUpdateMovable:
					bUpdateOverlaps = IsRootComponentMovable();
					break;

				case EActorUpdateOverlapsMethod::NeverUpdate:
					bUpdateOverlaps = false;
					break;

				case EActorUpdateOverlapsMethod::AlwaysUpdate:
				default:
					bUpdateOverlaps = true;
					break;
			}

			if (bUpdateOverlaps)
			{
				UpdateOverlaps(bGenerateOverlapEventsDuringLevelStreaming);
			}
		}
	}
}

void AActor::EnableInput(APlayerController* PlayerController)
{
	if (PlayerController)
	{
		// If it doesn't exist create it and bind delegates
		if (!InputComponent)
		{
			InputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass());
			InputComponent->RegisterComponent();
			InputComponent->bBlockInput = bBlockInput;
			InputComponent->Priority = InputPriority;

			UInputDelegateBinding::BindInputDelegatesWithSubojects(this, InputComponent);
		}
		else
		{
			// Make sure we only have one instance of the InputComponent on the stack
			PlayerController->PopInputComponent(InputComponent);
		}

		PlayerController->PushInputComponent(InputComponent);
	}
}

void AActor::CreateInputComponent(TSubclassOf<UInputComponent> InputComponentToCreate)
{
	if (InputComponentToCreate && !InputComponent)
	{
		InputComponent = NewObject<UInputComponent>(this, InputComponentToCreate);
		InputComponent->RegisterComponent();
		InputComponent->bBlockInput = bBlockInput;
		InputComponent->Priority = InputPriority;

		UInputDelegateBinding::BindInputDelegatesWithSubojects(this, InputComponent);
	}
}

void AActor::DisableInput(APlayerController* PlayerController)
{
	if (InputComponent)
	{
		if (PlayerController)
		{
			PlayerController->PopInputComponent(InputComponent);
		}
		else if (UWorld* World = GetWorld())
		{
			for (FConstPlayerControllerIterator PCIt = World->GetPlayerControllerIterator(); PCIt; ++PCIt)
			{
				if (APlayerController* PC = PCIt->Get())
				{
					PC->PopInputComponent(InputComponent);
				}
			}
		}
	}
}

float AActor::GetInputAxisValue(const FName InputAxisName) const
{
	float Value = 0.f;

	if (InputComponent)
	{
		Value = InputComponent->GetAxisValue(InputAxisName);
	}

	return Value;
}

float AActor::GetInputAxisKeyValue(const FKey InputAxisKey) const
{
	float Value = 0.f;

	if (InputComponent)
	{
		Value = InputComponent->GetAxisKeyValue(InputAxisKey);
	}

	return Value;
}

FVector AActor::GetInputVectorAxisValue(const FKey InputAxisKey) const
{
	FVector Value = FVector::ZeroVector;

	if (InputComponent)
	{
		Value = InputComponent->GetVectorAxisValue(InputAxisKey);
	}

	return Value;
}

bool AActor::SetActorLocation(const FVector& NewLocation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		const FVector Delta = NewLocation - GetActorLocation();
		return RootComponent->MoveComponent(Delta, GetActorQuat(), bSweep, OutSweepHitResult, MOVECOMP_NoFlags, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}

	return false;
}

bool AActor::SetActorRotation(FRotator NewRotation, ETeleportType Teleport)
{
#if ENABLE_NAN_DIAGNOSTIC
	if (NewRotation.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("AActor::SetActorRotation found NaN in FRotator NewRotation"));
		NewRotation = FRotator::ZeroRotator;
	}
#endif
	if (RootComponent)
	{
		return RootComponent->MoveComponent(FVector::ZeroVector, NewRotation, true, nullptr, MOVECOMP_NoFlags, Teleport);
	}

	return false;
}

bool AActor::SetActorRotation(const FQuat& NewRotation, ETeleportType Teleport)
{
#if ENABLE_NAN_DIAGNOSTIC
	if (NewRotation.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("AActor::SetActorRotation found NaN in FQuat NewRotation"));
	}
#endif
	if (RootComponent)
	{
		return RootComponent->MoveComponent(FVector::ZeroVector, NewRotation, true, nullptr, MOVECOMP_NoFlags, Teleport);
	}

	return false;
}

bool AActor::SetActorLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
#if ENABLE_NAN_DIAGNOSTIC
	if (NewRotation.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("AActor::SetActorLocationAndRotation found NaN in FRotator NewRotation"));
		NewRotation = FRotator::ZeroRotator;
	}
#endif
	if (RootComponent)
	{
		const FVector Delta = NewLocation - GetActorLocation();
		return RootComponent->MoveComponent(Delta, NewRotation, bSweep, OutSweepHitResult, MOVECOMP_NoFlags, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}

	return false;
}

bool AActor::SetActorLocationAndRotation(FVector NewLocation, const FQuat& NewRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
#if ENABLE_NAN_DIAGNOSTIC
	if (NewRotation.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("AActor::SetActorLocationAndRotation found NaN in FQuat NewRotation"));
	}
#endif
	if (RootComponent)
	{
		const FVector Delta = NewLocation - GetActorLocation();
		return RootComponent->MoveComponent(Delta, NewRotation, bSweep, OutSweepHitResult, MOVECOMP_NoFlags, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}

	return false;
}

void AActor::SetActorScale3D(FVector NewScale3D)
{
	if (RootComponent)
	{
		RootComponent->SetWorldScale3D(NewScale3D);
	}
}


FVector AActor::GetActorScale3D() const
{
	if (RootComponent)
	{
		return RootComponent->GetComponentScale();
	}
	return FVector(1,1,1);
}

void AActor::AddActorWorldOffset(FVector DeltaLocation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->AddWorldOffset(DeltaLocation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::AddActorWorldRotation(FRotator DeltaRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->AddWorldRotation(DeltaRotation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::AddActorWorldRotation(const FQuat& DeltaRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->AddWorldRotation(DeltaRotation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::AddActorWorldTransform(const FTransform& DeltaTransform, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->AddWorldTransform(DeltaTransform, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::AddActorWorldTransformKeepScale(const FTransform& DeltaTransform, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->AddWorldTransformKeepScale(DeltaTransform, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

bool AActor::SetActorTransform(const FTransform& NewTransform, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	// we have seen this gets NAN from kismet, and would like to see if this
	// happens, and if so, something else is giving NAN as output
	if (RootComponent)
	{
		if (ensureMsgf(!NewTransform.ContainsNaN(), TEXT("SetActorTransform: Get NAN Transform data for %s: %s"), *GetNameSafe(this), *NewTransform.ToString()))
		{
			RootComponent->SetWorldTransform(NewTransform, bSweep, OutSweepHitResult, Teleport);
		}
		else
		{
			if (OutSweepHitResult)
			{
				*OutSweepHitResult = FHitResult();
			}
		}
		return true;
	}

	if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
	return false;
}

void AActor::AddActorLocalOffset(FVector DeltaLocation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if(RootComponent)
	{
		RootComponent->AddLocalOffset(DeltaLocation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::AddActorLocalRotation(FRotator DeltaRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if(RootComponent)
	{
		RootComponent->AddLocalRotation(DeltaRotation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::AddActorLocalRotation(const FQuat& DeltaRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->AddLocalRotation(DeltaRotation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::AddActorLocalTransform(const FTransform& NewTransform, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if(RootComponent)
	{
		RootComponent->AddLocalTransform(NewTransform, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::SetActorRelativeLocation(FVector NewRelativeLocation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->SetRelativeLocation(NewRelativeLocation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::SetActorRelativeRotation(FRotator NewRelativeRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->SetRelativeRotation(NewRelativeRotation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::SetActorRelativeRotation(const FQuat& NewRelativeRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->SetRelativeRotation(NewRelativeRotation, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::SetActorRelativeTransform(const FTransform& NewRelativeTransform, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	if (RootComponent)
	{
		RootComponent->SetRelativeTransform(NewRelativeTransform, bSweep, OutSweepHitResult, Teleport);
	}
	else if (OutSweepHitResult)
	{
		*OutSweepHitResult = FHitResult();
	}
}

void AActor::SetActorRelativeScale3D(FVector NewRelativeScale)
{
	if (RootComponent)
	{
		if (NewRelativeScale.ContainsNaN())
		{
			FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("InvalidScale", "Scale '{0}' is not valid."), FText::FromString(NewRelativeScale.ToString())));
			return;
		}

		RootComponent->SetRelativeScale3D(NewRelativeScale);
	}
}

FVector AActor::GetActorRelativeScale3D() const
{
	if (RootComponent)
	{
		return RootComponent->GetRelativeScale3D();
	}
	return FVector(1,1,1);
}

void AActor::MarkNeedsRecomputeBoundsOnceForGame()
{
	ForEachComponent<USceneComponent>(true, [](USceneComponent* SceneComponent)
	{
		if (SceneComponent && SceneComponent->bComputeBoundsOnceForGame)
		{
			SceneComponent->bComputedBoundsOnceForGame = false;
		}
	});
}

void AActor::SetActorHiddenInGame( bool bNewHidden )
{
	if (IsHidden() != bNewHidden)
	{
		SetHidden(bNewHidden);
		UpdateComponentVisibility();
	}
}

void AActor::SetActorEnableCollision(bool bNewActorEnableCollision)
{
	if(bActorEnableCollision != bNewActorEnableCollision)
	{
		bActorEnableCollision = bNewActorEnableCollision;

		// Notify components about the change
		TInlineComponentArray<UActorComponent*> Components;
		GetComponents(Components);

		for(int32 CompIdx=0; CompIdx<Components.Num(); CompIdx++)
		{
			Components[CompIdx]->OnActorEnableCollisionChanged();
		}

		// update overlaps once after all components have been updated
		UpdateOverlaps();
	}
}


bool AActor::Destroy( bool bNetForce, bool bShouldModifyLevel )
{
	// It's already pending kill or in DestroyActor(), no need to beat the corpse
	if (!IsPendingKillPending())
	{
		UWorld* World = GetWorld();
		if (World)
		{
			World->DestroyActor( this, bNetForce, bShouldModifyLevel );
		}
		else
		{
			UE_LOGF(LogSpawn, Warning, "Destroying %ls, which doesn't have a valid world pointer", *GetPathName());
		}
	}

	return IsPendingKillPending();
}

void AActor::K2_DestroyActor()
{
	Destroy();
}

bool AActor::SetRootComponent(class USceneComponent* NewRootComponent)
{
	/** Only components owned by this actor can be used as a its root component. */
	if (ensure(NewRootComponent == nullptr || NewRootComponent->GetOwner() == this))
	{
		if (RootComponent != NewRootComponent)
		{
			Modify();

			USceneComponent* OldRootComponent = RootComponent;
			RootComponent = NewRootComponent;

			// Notify new root first, as it probably has no delegate on it.
			if (NewRootComponent)
			{
				NewRootComponent->NotifyIsRootComponentChanged(true);
			}

			if (OldRootComponent)
			{
				OldRootComponent->NotifyIsRootComponentChanged(false);
			}
		}
		return true;
	}

	return false;
}

void AActor::GetActorBounds(bool bOnlyCollidingComponents, FVector& Origin, FVector& BoxExtent, bool bIncludeFromChildActors) const
{
	const FBox Bounds = GetComponentsBoundingBox(!bOnlyCollidingComponents, bIncludeFromChildActors);

	// To keep consistency with the other GetBounds functions, transform our result into an origin / extent formatting
	Bounds.GetCenterAndExtents(Origin, BoxExtent);
}


AWorldSettings * AActor::GetWorldSettings() const
{
	UWorld* World = GetWorld();
	return (World ? World->GetWorldSettings() : nullptr);
}

UNetDriver* GetNetDriver_Internal(UWorld* World, FName NetDriverName)
{
	if (NetDriverName == NAME_GameNetDriver)
	{
		return (World ? World->GetNetDriver() : nullptr);
	}

	return GEngine->FindNamedNetDriver(World, NetDriverName);
}

// Note: this is a private implementation that should no t be called directly except by the public wrappers (GetNetMode()) where some optimizations are inlined.
ENetMode AActor::InternalGetNetMode() const
{
	UWorld* World = GetWorld();
	UNetDriver* NetDriver = GetNetDriver_Internal(World, NetDriverName);
	if (NetDriver != nullptr)
	{
		return NetDriver->GetNetMode();
	}

	if (World)
	{
		// World handles the demo net mode and has some special case checks for PIE
		return World->GetNetMode();
	}

	return NM_Standalone;
}

UNetDriver* AActor::GetNetDriver() const
{
	return GetNetDriver_Internal(GetWorld(), NetDriverName);
}

void AActor::SetNetDriverName(FName NewNetDriverName)
{
	if (NewNetDriverName != NetDriverName)
	{
		UWorld* World = GetWorld();
		if (World)
		{
			World->RemoveNetworkActor(this);
		}

		NetDriverName = NewNetDriverName;

		if (World)
		{
			World->AddNetworkActor(this);
		}
	}
}

//
// Return whether a function should be executed remotely.
//
int32 AActor::GetFunctionCallspace( UFunction* Function, FFrame* Stack )
{
	if (GAllowActorScriptExecutionInEditor)
	{
		// Call local, this global is only true when we know it's being called on an editor-placed object
		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace ScriptExecutionInEditor: %s"), *Function->GetName());
		return FunctionCallspace::Local;
	}

	if ((Function->FunctionFlags & FUNC_Static) || (GetWorld() == nullptr))
	{
		// Use the same logic as function libraries for static/CDO called functions, will try to use the global context to check authority only/cosmetic
		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Static: %s"), *Function->GetName());

		return GEngine->GetGlobalFunctionCallspace(Function, this, Stack);
	}

	const ENetRole LocalRole = GetLocalRole();

	// If we are on a client and function is 'skip on client', absorb it
	FunctionCallspace::Type Callspace = (LocalRole < ROLE_Authority) && Function->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly) ? FunctionCallspace::Absorbed : FunctionCallspace::Local;
	
	if (!IsValidChecked(this))
	{
		// Never call remote on a pending kill actor. 
		// We can call it local or absorb it depending on authority/role check above.
		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace: IsPendingKill %s %s"), *Function->GetName(), FunctionCallspace::ToString(Callspace));
		return Callspace;
	}

	if (Function->FunctionFlags & FUNC_NetRequest)
	{
		// Call remote
		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace NetRequest: %s"), *Function->GetName());
		return FunctionCallspace::Remote;
	}	
	
	if (Function->FunctionFlags & FUNC_NetResponse)
	{
		if (Function->RPCId > 0)
		{
			// Call local
			DEBUG_CALLSPACE(TEXT("GetFunctionCallspace NetResponse Local: %s"), *Function->GetName());
			return FunctionCallspace::Local;
		}

		// Shouldn't happen, so skip call
		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace NetResponse Absorbed: %s"), *Function->GetName());
		return FunctionCallspace::Absorbed;
	}

	const ENetMode NetMode = GetNetMode();
	// Quick reject 2. Has to be a network game to continue
	if (NetMode == NM_Standalone)
	{
		if (LocalRole < ROLE_Authority && (Function->FunctionFlags & FUNC_NetServer))
		{
			// Don't let clients call server functions (in edge cases where NetMode is standalone (net driver is null))
			DEBUG_CALLSPACE(TEXT("GetFunctionCallspace No Authority Server Call Absorbed: %s"), *Function->GetName());
			return FunctionCallspace::Absorbed;
		}

		// Call local
		return FunctionCallspace::Local;
	}
	
	// Dedicated servers don't care about "cosmetic" functions.
	if (NetMode == NM_DedicatedServer && Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic))
	{
		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Blueprint Cosmetic Absorbed: %s"), *Function->GetName());
		return FunctionCallspace::Absorbed;
	}

	if (!(Function->FunctionFlags & FUNC_Net))
	{
		// Not a network function
		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Not Net: %s %s"), *Function->GetName(), FunctionCallspace::ToString(Callspace));
		return Callspace;
	}
	
	bool bIsServer = NetMode == NM_ListenServer || NetMode == NM_DedicatedServer;

	// get the top most function
	while (Function->GetSuperFunction() != nullptr)
	{
		Function = Function->GetSuperFunction();
	}

	if ((Function->FunctionFlags & FUNC_NetMulticast))
	{
		if(bIsServer)
		{
			// Server should execute locally and call remotely
			if (RemoteRole != ROLE_None)
			{
				DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Multicast: %s"), *Function->GetName());
				return (FunctionCallspace::Local | FunctionCallspace::Remote);
			}

			DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Multicast NoRemoteRole: %s"), *Function->GetName());
			return FunctionCallspace::Local;
		}
		else
		{
			// Client should only execute locally iff it is allowed to (function is not KismetAuthorityOnly)
			DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Multicast Client: %s %s"), *Function->GetName(), FunctionCallspace::ToString(Callspace));
			return Callspace;
		}
	}

	if (Function->FunctionFlags & (FUNC_NetServer | FUNC_NetClient | FUNC_NetMulticast))
	{
		// Handle uni-directional RPCs

		// if we are the server, and it's not a send-to-client function,
		if (bIsServer && !(Function->FunctionFlags & FUNC_NetClient))
		{
			// don't replicate
			DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Server calling Server function: %s %s"), *Function->GetName(), FunctionCallspace::ToString(Callspace));
			return Callspace;
		}
		// if we aren't the server, and it's not a send-to-server function,
		if (!bIsServer && !(Function->FunctionFlags & FUNC_NetServer))
		{
			// don't replicate
			DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Client calling Client function: %s %s"), *Function->GetName(), FunctionCallspace::ToString(Callspace));
			return Callspace;
		}
	}
	else if (UE::Net::Private::FCoreNetContext::Get()->GetCurrentRemoteFunctionMode() == UE::Net::Private::ERemoteFunctionMode::Receiving)
	{
		// Received a function with FUNC_Net but not Client, Server, or Multicast (Remote specifier) - can be sent bi-directionally
		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Received a Remote function, calling: %s %s"), *Function->GetName(), FunctionCallspace::ToString(Callspace));
		return Callspace;
	}

	// Check if the actor can potentially call remote functions	
	if (LocalRole == ROLE_Authority)
	{
		UNetConnection* NetConnection = GetNetConnection();
		if (NetConnection == nullptr)
		{
			UPlayer *ClientPlayer = GetNetOwningPlayer();
			if (ClientPlayer == nullptr)
			{
				// Check if a player ever owned this (topmost owner is playercontroller or beacon)
				if (HasNetOwner())
				{
					// Network object with no owning player, we must absorb
					DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Client without owner absorbed %s"), *Function->GetName());
					return FunctionCallspace::Absorbed;
				}

				if ((Function->FunctionFlags & (FUNC_NetServer | FUNC_NetClient | FUNC_NetMulticast)) == 0)
				{
					// Calling a Remote function (client or server) with no owning player, absorb these.
					UE_LOGF(LogNet, Error, "GetFunctionCallspace calling a Remote function without owner, absorbing: %ls", *Function->GetName());
					return FunctionCallspace::Absorbed;
				}

				// Role authority object calling a client RPC locally (ie AI owned objects)
				DEBUG_CALLSPACE(TEXT("GetFunctionCallspace authority non client owner %s %s"), *Function->GetName(), FunctionCallspace::ToString(Callspace));
				return Callspace;
			}
			else if (Cast<ULocalPlayer>(ClientPlayer) != nullptr)
			{
				// This is a local player, call locally
				DEBUG_CALLSPACE(TEXT("GetFunctionCallspace Client local function: %s %s"), *Function->GetName(), FunctionCallspace::ToString(Callspace));
				return Callspace;
			}
		}
		else if (!NetConnection->Driver || !NetConnection->Driver->World)
		{
			// NetDriver does not have a world, most likely shutting down
			DEBUG_CALLSPACE(TEXT("GetFunctionCallspace NetConnection with no driver or world absorbed: %s %s %s"),
				*Function->GetName(), 
				NetConnection->Driver ? *NetConnection->Driver->GetName() : TEXT("NoNetDriver"),
				NetConnection->Driver && NetConnection->Driver->World ? *NetConnection->Driver->World->GetName() : TEXT("NoWorld"));
			return FunctionCallspace::Absorbed;
		}

		// There is a valid net connection, so continue and call remotely
	}

	// about to call remotely - unless the actor is not actually replicating
	if (RemoteRole == ROLE_None)
	{
		if (!bIsServer)
		{
			UE_LOGF(LogNet, Warning, "Client is absorbing remote function %ls on actor %ls because RemoteRole is ROLE_None", *Function->GetName(), *GetName() );
		}

		DEBUG_CALLSPACE(TEXT("GetFunctionCallspace RemoteRole None absorbed %s"), *Function->GetName());
		return FunctionCallspace::Absorbed;
	}

	// Call remotely
	DEBUG_CALLSPACE(TEXT("GetFunctionCallspace RemoteRole Remote %s"), *Function->GetName());
	return FunctionCallspace::Remote;
}

bool AActor::CallRemoteFunction( UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack )
{
	bool bProcessed = false;

	if (UWorld* MyWorld = GetWorld())
	{
		if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(MyWorld))
		{
			for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
			{
				if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateFunction(this, Function))
				{
					Driver.NetDriver->ProcessRemoteFunction(this, Function, Parameters, OutParms, Stack, nullptr);
					bProcessed = true;
				}
			}
		}
	}

	return bProcessed;
}

void AActor::DispatchPhysicsCollisionHit(const FRigidBodyCollisionInfo& MyInfo, const FRigidBodyCollisionInfo& OtherInfo, const FCollisionImpactData& RigidCollisionData)
{
#if 0
	if(true)
	{
		FString MyName = MyInfo.Component ? FString(*MyInfo.Component->GetPathName()) : FString();
		FString OtherName = OtherInfo.Component ? FString(*OtherInfo.Component->GetPathName()) : FString();
		UE_LOGF(LogPhysics, Log,  "COLLIDE! %ls - %ls", *MyName, *OtherName );
	}
#endif

	checkSlow(RigidCollisionData.ContactInfos.Num() > 0);

	// @todo At the moment we only pass the first contact in the ContactInfos array. Maybe improve this?
	const FRigidBodyContactInfo& ContactInfo = RigidCollisionData.ContactInfos[0];

	FHitResult Result;
	Result.Location = Result.ImpactPoint = ContactInfo.ContactPosition;
	Result.Normal = Result.ImpactNormal = ContactInfo.ContactNormal;
	Result.PenetrationDepth = ContactInfo.ContactPenetration;
	Result.PhysMaterial = ContactInfo.PhysMaterial[1];
	Result.Component = OtherInfo.Component;
	Result.MyItem = MyInfo.BodyIndex;
	Result.Item = OtherInfo.BodyIndex;
	Result.BoneName = OtherInfo.BoneName;
	Result.MyBoneName = MyInfo.BoneName;
	Result.bBlockingHit = true;

	AActor* Actor = OtherInfo.Actor.Get();
	UPrimitiveComponent* Component = OtherInfo.Component.Get();
	Result.HitObjectHandle = FActorInstanceHandle(Actor, Component, OtherInfo.BodyIndex);

	NotifyHit(MyInfo.Component.Get(), Actor, Component, true, Result.Location, Result.Normal, RigidCollisionData.TotalNormalImpulse, Result);

	// Execute delegates if bound

	if (OnActorHit.IsBound())
	{
		OnActorHit.Broadcast(this, OtherInfo.Actor.Get(), RigidCollisionData.TotalNormalImpulse, Result);
	}

	UPrimitiveComponent* MyInfoComponent = MyInfo.Component.Get();
	if (MyInfoComponent && MyInfoComponent->OnComponentHit.IsBound())
	{
		MyInfoComponent->OnComponentHit.Broadcast(MyInfoComponent, OtherInfo.Actor.Get(), OtherInfo.Component.Get(), RigidCollisionData.TotalNormalImpulse, Result);
	}
}

#if WITH_EDITOR
bool AActor::IsTemporarilyHiddenInEditor(const bool bIncludeParent) const
{
	if (bHiddenEdTemporary)
	{
		return true;
	}

	if (bIncludeParent)
	{
		if (UChildActorComponent* ParentCAC = ParentComponent.Get())
		{
			return ParentCAC->GetOwner()->IsTemporarilyHiddenInEditor(true);
		}
	}

	return false;
}
#endif

bool AActor::IsSelectionParentOfAttachedActors() const
{
	return false;
}

bool AActor::IsSelectionChild() const
{
	if (IsChildActor())
	{
		return true;
	}

	const AActor* AttachParent = GetAttachParentActor();
	while (AttachParent != nullptr)
	{
		if (AttachParent->IsSelectionParentOfAttachedActors())
		{
			return true;
		}

		AttachParent = AttachParent->GetAttachParentActor();
	}

	return false;
}

AActor* AActor::GetSelectionParent() const
{
	if (!UChildActorComponent::ExperimentalAllowPerInstanceChildActorProperties() && IsChildActor())
	{
		return GetParentActor();
	}

	AActor* AttachParent = GetAttachParentActor();
	while (AttachParent != nullptr)
	{
		if (AttachParent->IsSelectionParentOfAttachedActors())
		{
			return AttachParent;
		}

		AttachParent = AttachParent->GetAttachParentActor();
	}

	return nullptr;
}

bool AActor::SupportsSubRootSelection() const
{
#if WITH_EDITOR
	if (IsInLevelInstance())
	{
		return UWorld::GetSubsystem<ULevelInstanceSubsystem>(GetWorld())->IsSubSelectionEnabled();
	}
#endif

	return false;
}

AActor* AActor::GetRootSelectionParent() const
{
	AActor* Parent = GetSelectionParent();
	while (Parent != nullptr && Parent->IsSelectionChild())
	{
		Parent = Parent->GetSelectionParent();
	}
	return Parent;
}

bool AActor::IsActorOrSelectionParentSelected() const
{
	return IsSelected() || (GetSelectionParent() && GetSelectionParent()->IsActorOrSelectionParentSelected());
}

void AActor::PushSelectionToProxies()
{
	TInlineComponentArray<UActorComponent*> Components;
	GetComponents(Components);

	//for every component in the actor
	for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
	{
		UActorComponent* Component = Components[ComponentIndex];
		if (Component->IsRegistered())
		{
			if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(Component))
			{
				if (AActor* ChildActor = ChildActorComponent->GetChildActor())
				{
					ChildActor->PushSelectionToProxies();
				}
			}
			
			if(UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component))
			{
				PrimComponent->PushSelectionToProxy();
			}

			if (ULightComponent* LightComponent = Cast<ULightComponent>(Component))
			{
				LightComponent->PushSelectionToProxy();
			}
		}
	}

	if (IsSelectionParentOfAttachedActors())
	{
		TFunction<bool(AActor*)> PushAllChildrenToProxies = [&PushAllChildrenToProxies](AActor* AttachedActor)
		{
			AttachedActor->PushSelectionToProxies();
			if (!AttachedActor->IsSelectionParentOfAttachedActors())
			{
				AttachedActor->ForEachAttachedActors(PushAllChildrenToProxies);
			}
			return true;
		};

		ForEachAttachedActors(PushAllChildrenToProxies);
	}
}

#if WITH_EDITOR
void AActor::PushLevelInstanceEditingStateToProxies(bool bInEditingState)
{
	TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
	GetComponents(PrimComponents);

	if (bInEditingState)
	{
		EnumAddFlags(LevelInstanceFlags, ELevelInstanceFlags::IsInEditHierarchy);
	}
	else
	{
		EnumRemoveFlags(LevelInstanceFlags, ELevelInstanceFlags::IsInEditHierarchy);
	}

	for (const auto& PrimComponent : PrimComponents)
	{
		if (PrimComponent->IsRegistered())
		{
			PrimComponent->PushLevelInstanceEditingStateToProxy(bInEditingState);
		}
	}

	TFunction<bool(AActor*)> PushAllChildrenToProxies = [&PushAllChildrenToProxies, bInEditingState](AActor* AttachedActor)
	{
		AttachedActor->PushLevelInstanceEditingStateToProxies(bInEditingState);
		return true;
	};

	ForEachAttachedActors(PushAllChildrenToProxies);
}
#endif

bool AActor::IsChildActor() const
{
	return ParentComponent.IsValid();
}

UChildActorComponent* AActor::GetParentComponent() const
{
	return ParentComponent.Get();
}

AActor* AActor::GetParentActor() const
{
	AActor* ParentActor = nullptr;
	if (UChildActorComponent* ParentComponentPtr = GetParentComponent())
	{
		ParentActor = ParentComponentPtr->GetOwner();
	}

	return ParentActor;
}

void AActor::GetAllChildActors(TArray<AActor*>& ChildActors, bool bIncludeDescendants) const
{
	TInlineComponentArray<UChildActorComponent*> ChildActorComponents(this);

	ChildActors.Reserve(ChildActors.Num() + ChildActorComponents.Num());
	for (UChildActorComponent* CAC : ChildActorComponents)
	{
		if (AActor* ChildActor = CAC->GetChildActor())
		{
			ChildActors.Add(ChildActor);
			if (bIncludeDescendants)
			{
				ChildActor->GetAllChildActors(ChildActors, true);
			}
		}
	}
}


// COMPONENTS

void AActor::UnregisterAllComponents(const bool bForReregister)
{
	// This function may be called multiple times for each actor at different states of destruction:
	// Use the cached all components registration flag to ensure the world is only notified once.
	if (bHasRegisteredAllComponents)
	{
		if (UWorld* OwningWorld = GetWorld())
		{
			OwningWorld->NotifyPreUnregisterAllActorComponents(this);
		}
	}

	TInlineComponentArray<UActorComponent*> Components;
	GetComponents(Components);

	for(UActorComponent* Component : Components)
	{
		if( Component->IsRegistered() && (!bForReregister || Component->AllowReregistration())) // In some cases unregistering one component can unregister another, so we do a check here to avoid trying twice
		{
			Component->UnregisterComponent();
		}
	}

	if (bHasRegisteredAllComponents || GOptimizeActorRegistration == 0)
	{
		// With registration optimizations enabled, we need to make sure it unregisters components that were partially registered during construction,
		// but we do not want to call PostUnregisterAllComponents if it was only partially registered
		bHasRegisteredAllComponents = false;

		PostUnregisterAllComponents();
	}

	bHasPreRegisteredAllComponents = false;
}

void AActor::PostUnregisterAllComponents()
{
	ensureMsgf(bHasRegisteredAllComponents == false, TEXT("bHasRegisteredAllComponents must be set to false prior to calling PostUnregisterAllComponents()"));

	FNavigationSystem::OnActorUnregistered(*this);
}

void AActor::RegisterAllComponents()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AActor_RegisterAllComponents);
	
	PreRegisterAllComponents();
	ensureMsgf(HasPreRegisteredAllComponents(), TEXT("Actor %s PreRegisterAllComponents implementation seems be to missing a call to AActor::PreRegisterAllComponents."), *this->GetFullName());

	// 0 - means register all components
	bool bAllRegistered = IncrementalRegisterComponents(0);
	ensureMsgf(bAllRegistered, TEXT("Actor %s RegisterAllComponents failed to register some components."), *this->GetFullName());
}

namespace UE
{
namespace Private
{
// Walks through components hierarchy and returns closest to root parent component that is unregistered
// Only for components that belong to the same owner
static USceneComponent* GetUnregisteredParent(UActorComponent* Component)
{
	USceneComponent* ParentComponent = nullptr;
	USceneComponent* SceneComponent = Cast<USceneComponent>(Component);
	
	while (	SceneComponent && 
			SceneComponent->GetAttachParent() &&
			SceneComponent->GetAttachParent()->GetOwner() == Component->GetOwner() &&
			!SceneComponent->GetAttachParent()->IsRegistered())
	{
		SceneComponent = SceneComponent->GetAttachParent();
		if (SceneComponent->bAutoRegister && IsValidChecked(SceneComponent))
		{
			// We found unregistered parent that should be registered
			// But keep looking up the tree
			ParentComponent = SceneComponent;
		}
	}

	return ParentComponent;
}

static USceneComponent* GetNonPreRegisteredParent(UActorComponent* Component)
{
	USceneComponent* ParentComponent = nullptr;
	USceneComponent* SceneComponent = Cast<USceneComponent>(Component);

	while (SceneComponent && 
		   SceneComponent->GetAttachParent() && 
		   SceneComponent->GetAttachParent()->GetOwner() == Component->GetOwner() && 
		   !SceneComponent->GetAttachParent()->IsPreRegistering() &&
		   !SceneComponent->GetAttachParent()->IsPreRegistered())
	{
		SceneComponent = SceneComponent->GetAttachParent();
		if (SceneComponent->bAutoRegister && IsValidChecked(SceneComponent))
		{
			// We found non-preregistered parent that should be preregistered
			// Keep looking up the tree
			ParentComponent = SceneComponent;
		}
	}

	return ParentComponent;
}
} // namespace Private
} // namespace UE

bool AActor::SupportsIncrementalPreUnregisterComponents() const
{
	// Do not support incremental pre-unregister of components when a lifespan is set, 
	// as this could cause UActorComponent::DestroyPhysicsState to be called 
	// while the component is still in the pre-unregistering state (UActorComponent::IsPreUnregistering == true).
	return !TimerHandle_LifeSpanExpired.IsValid();
}

void AActor::IncrementalPreUnregisterComponents()
{
	if (!SupportsIncrementalPreUnregisterComponents())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AActor::IncrementalPreUnregisterComponents);

	TInlineComponentArray<UActorComponent*> Components;
	for (UActorComponent* Component : GetComponents())
	{
		if (Component && Component->ShouldIncrementalPreUnregister())
		{
			Components.Add(Component);
		}
	}

	if (!Components.Num())
	{
		return;
	}

	// Unregister all tick functions before pre-unregistering components as this will start asynchronous tasks (like destruction of the physics state)
	RegisterAllActorTickFunctions(false, true); 

	for (UActorComponent* Component : Components)
	{
		Component->PreUnregisterComponentFromWorld();
	}
}

void AActor::IncrementalPreRegisterComponents()
{
	if (!SupportsIncrementalPreRegisterComponents())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AActor::IncrementalPreRegisterComponents);

	// First call PreRegisterAllComponents (IncrementalPreRegisterComponents includes PreRegisterAllComponents)
#if WITH_EDITOR
	// In PIE, WorldSettings is spawned when world partition cell level is created, AActor::PostSpawnInitialize will 
	// have already called RegisterAllComponents (which calls PreRegisterAllComponents)
	if (!HasPreRegisteredAllComponents())
#endif
	{
		PreRegisterAllComponents();
	}
	ensureMsgf(HasPreRegisteredAllComponents(), TEXT("Actor %s PreRegisterAllComponents implementation seems be to missing a call to AActor::PreRegisterAllComponents."), *this->GetFullName());

	UWorld* const World = GetWorld();
	TInlineComponentArray<UActorComponent*> Components;
	for (UActorComponent* Component : GetComponents())
	{
		if (Component)
		{
#if WITH_EDITOR
			// Ignore registered components (can happen if construction script adds component(s), which happens before this)
			if (!Component->IsRegistered())
#endif
			{
				if (Component->ShouldIncrementalPreRegister(World))
				{
					Components.Add(Component);
				}
			}
		}
	}

	// No component found to pre-register
	if (Components.IsEmpty())
	{
		return;
	}

	// Pre-register RootComponent first so all other children components can reliably use it (i.e., call GetLocation) when they pre-register
	if (RootComponent && RootComponent->bAutoRegister && !RootComponent->IsPreRegistered())
	{
#if WITH_EDITOR
		// Ignore registered components (can happen if construction script adds component(s), which happens before this)
		if (!RootComponent->IsRegistered())
#endif 
		{
			check(!RootComponent->IsPreRegistering());
			RootComponent->PreRegisterComponentWithWorld(World);
		}
	}

	TSet<UActorComponent*> PreRegisteredParents;
	for (int32 CompIdx = 0; CompIdx < Components.Num(); CompIdx++)
	{
		UActorComponent* Component = Components[CompIdx];
		if (!Component->IsPreRegistered() && !Component->IsPreRegistering() && Component->bAutoRegister && IsValidChecked(Component))
		{
			// Ensure that all parents are pre-registered first
			USceneComponent* NonPreRegisteredParentComponent = UE::Private::GetNonPreRegisteredParent(Component);
			if (NonPreRegisteredParentComponent)
			{
				bool bParentAlreadyHandled = false;
				PreRegisteredParents.Add(NonPreRegisteredParentComponent, &bParentAlreadyHandled);
				if (bParentAlreadyHandled)
				{
					UE_LOGF(LogActor, Error, "AActor::IncrementalPreRegisterComponents parent component '%ls' cannot be pre-registered in actor '%ls'", *GetPathNameSafe(NonPreRegisteredParentComponent), *GetPathName());
					break;
				}

				// Register parent first, then return to this component on a next iteration
				Component = NonPreRegisteredParentComponent;
				CompIdx--;
			}
			Component->PreRegisterComponentWithWorld(World);
		}
	}
}

bool AActor::IncrementalRegisterComponents(int32 NumComponentsToRegister, FRegisterComponentContext* Context)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET(this, GetClass());
	if (NumComponentsToRegister == 0)
	{
		// 0 - means register all components
		NumComponentsToRegister = MAX_int32;
	}

	UWorld* const World = GetWorld();
	check(World);

	// If we are not a game world, then register tick functions now. If we are a game world we wait until right before BeginPlay(),
	// so as to not actually tick until BeginPlay() executes (which could otherwise happen in network games).
	if (bAllowTickBeforeBeginPlay || !World->IsGameWorld())
	{
		RegisterAllActorTickFunctions(true, false); // components will be handled when they are registered
	}
	
	// Register RootComponent first so all other children components can reliably use it (i.e., call GetLocation) when they register
	if (RootComponent != nullptr && !RootComponent->IsRegistered())
	{
#if PERF_TRACK_DETAILED_ASYNC_STATS
		FScopeCycleCounterUObject ContextScope(RootComponent);
#endif
		if (RootComponent->bAutoRegister)
		{
			// Before we register our component, save it to our transaction buffer so if "undone" it will return to an unregistered state.
			// This should prevent unwanted components hanging around when undoing a copy/paste or duplication action.
			RootComponent->Modify(false);

			RootComponent->RegisterComponentWithWorld(World, Context);
		}
	}

	int32 NumTotalRegisteredComponents = 0;
	int32 NumRegisteredComponentsThisRun = 0;
	TInlineComponentArray<UActorComponent*> Components;
	GetComponents(Components);
	const int32 ComponentCount = Components.Num();
	TSet<UActorComponent*> RegisteredParents;

	const UE::FTimeout* Timeout = Context ? &Context->GetIncrementalUpdateTimeout() : nullptr;

	for (int32 CompIdx = 0; CompIdx < ComponentCount; CompIdx++)
	{
		UActorComponent* Component = Components[CompIdx];
		if (!Component->IsRegistered() && Component->bAutoRegister && IsValidChecked(Component))
		{
			// Ensure that all parent are registered first
			USceneComponent* UnregisteredParentComponent = UE::Private::GetUnregisteredParent(Component);
			if (UnregisteredParentComponent)
			{
				bool bParentAlreadyHandled = false;
				RegisteredParents.Add(UnregisteredParentComponent, &bParentAlreadyHandled);
				if (bParentAlreadyHandled)
				{
					UE_LOGF(LogActor, Error, "AActor::IncrementalRegisterComponents parent component '%ls' cannot be registered in actor '%ls'", *GetPathNameSafe(UnregisteredParentComponent), *GetPathName());
					break;
				}

				// Register parent first, then return to this component on a next iteration
				Component = UnregisteredParentComponent;
				CompIdx--;
				NumTotalRegisteredComponents--; // because we will try to register the parent again later...
			}
#if PERF_TRACK_DETAILED_ASYNC_STATS
			FScopeCycleCounterUObject ContextScope(Component);
#endif
				
			// Before we register our component, save it to our transaction buffer so if "undone" it will return to an unregistered state.
			// This should prevent unwanted components hanging around when undoing a copy/paste or duplication action.
			Component->Modify(false);

			Component->RegisterComponentWithWorld(World, Context);
			NumRegisteredComponentsThisRun++;
		}

		NumTotalRegisteredComponents++;

		if ((CompIdx + 1 < ComponentCount) && (NumRegisteredComponentsThisRun >= NumComponentsToRegister))
		{
			NumRegisteredComponentsThisRun = 0;
			if (Timeout == nullptr)
			{
				// If there is no time limit, stop at NumComponentsToRegister
				break;
			}
			else if (Timeout->IsExpired())
			{
				// If there is a time limit, only stop if we're out of time
				break;
			}
		}
	}

	// See whether we are done
	if (ComponentCount == NumTotalRegisteredComponents)
	{
#if PERF_TRACK_DETAILED_ASYNC_STATS
		QUICK_SCOPE_CYCLE_COUNTER(STAT_AActor_IncrementalRegisterComponents_PostRegisterAllComponents);
#endif
		
		// Skip the world post register if optimizations are enabled and it was already called
		const bool bCallWorldPostRegister = (!bHasRegisteredAllComponents || GOptimizeActorRegistration == 0);

		// Clear this flag as it's no longer deferred
		bHasDeferredComponentRegistration = false;

		bHasRegisteredAllComponents = true;
		// Finally, call PostRegisterAllComponents
		PostRegisterAllComponents();

		if (bCallWorldPostRegister)
		{
			// After all components have been registered the actor is considered fully added: notify the owning world.
			World->NotifyPostRegisterAllActorComponents(this);
		}
		return true;
	}
	
	// Still have components to register
	return false;
}

bool AActor::HasValidRootComponent() const
{ 
	return (RootComponent != nullptr && RootComponent->IsRegistered()); 
}

void AActor::MarkComponentsAsGarbage(bool bModify)
{
	// Iterate components and mark them all as garbage.
	TInlineComponentArray<UActorComponent*> Components(this);

	for (UActorComponent* Component : Components)
	{
		// Modify component so undo/ redo works in the editor.
		if (bModify && GIsEditor)
		{
			Component->Modify();
		}
		Component->OnComponentDestroyed(true);
		Component->MarkAsGarbage();
	}
}

void AActor::ReregisterAllComponents()
{
	UnregisterAllComponents(true);
	RegisterAllComponents();
}

void AActor::UpdateComponentTransforms()
{
	for (UActorComponent* ActorComp : GetComponents())
	{
		if (ActorComp && ActorComp->IsRegistered())
		{
			ActorComp->UpdateComponentToWorld();
		}
	}
}

void AActor::UpdateComponentVisibility()
{
	for (UActorComponent* ActorComp : GetComponents())
	{
		if (ActorComp && ActorComp->IsRegistered())
		{
			ActorComp->OnActorVisibilityChanged();
			if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(ActorComp))
			{
				if (ChildActorComponent->GetChildActor())
				{
					ChildActorComponent->GetChildActor()->UpdateComponentVisibility();
				}
			}
		}
	}
}

void AActor::MarkComponentsRenderStateDirty()
{
	for (UActorComponent* ActorComp : GetComponents())
	{
		if (ActorComp && ActorComp->IsRegistered())
		{
			ActorComp->MarkRenderStateDirty();
			if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(ActorComp))
			{
				if (ChildActorComponent->GetChildActor())
				{
					ChildActorComponent->GetChildActor()->MarkComponentsRenderStateDirty();
				}
			}
		}
	}
}

void AActor::InitializeComponents()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Actor_InitializeComponents);

	TInlineComponentArray<UActorComponent*> Components;
	GetComponents(Components);

	for (UActorComponent* ActorComp : Components)
	{
		if (ActorComp->IsRegistered())
		{
			if (ActorComp->bAutoActivate && !ActorComp->IsActive())
			{
				ActorComp->Activate(true);
			}

			if (ActorComp->bWantsInitializeComponent && !ActorComp->HasBeenInitialized())
			{
				// Broadcast the activation event since Activate occurs too early to fire a callback in a game
				ActorComp->InitializeComponent();
			}
		}
	}
}

void AActor::UninitializeComponents()
{
	TInlineComponentArray<UActorComponent*> Components;
	GetComponents(Components);

	for (UActorComponent* ActorComp : Components)
	{
		if (ActorComp->HasBeenInitialized())
		{
			ActorComp->UninitializeComponent();
		}
	}
}

void AActor::HandleRegisterComponentWithWorld(UActorComponent* Component)
{
	const bool bOwnerBeginPlayStarted = HasActorBegunPlay() || IsActorBeginningPlay();

	if (!Component->HasBeenInitialized() && Component->bWantsInitializeComponent && IsActorInitialized())
	{
		Component->InitializeComponent();

		// The component was finally initialized, it can now be replicated
		// Note that if this component does not ask to be initialized, it would have started to be replicated inside AddOwnedComponent.
		if (bOwnerBeginPlayStarted && Component->GetIsReplicated())
		{
			AddComponentForReplication(Component);
		}
	}

	if (bOwnerBeginPlayStarted)
	{
		Component->RegisterAllComponentTickFunctions(true);

		if (!Component->HasBegunPlay())
		{
			Component->BeginPlay();
			ensureMsgf(Component->HasBegunPlay(), TEXT("Failed to route BeginPlay (%s)"), *Component->GetFullName());
		}
	}
}

void AActor::DrawDebugComponents(FColor const& BaseColor) const
{
#if ENABLE_DRAW_DEBUG
	UWorld* MyWorld = GetWorld();

	for (UActorComponent* ActorComp : GetComponents())
	{
		if (USceneComponent const* const Component = Cast<USceneComponent>(ActorComp))
		{
			FVector const Loc = Component->GetComponentLocation();
			FRotator const Rot = Component->GetComponentRotation();

			// draw coord system at component loc
			DrawDebugCoordinateSystem(MyWorld, Loc, Rot, 10.f);

			// draw line from me to my parent
			if (Component->GetAttachParent())
			{
				DrawDebugLine(MyWorld, Component->GetAttachParent()->GetComponentLocation(), Loc, BaseColor);
			}

			// draw component name
			DrawDebugString(MyWorld, Loc+FVector(0,0,32), *Component->GetName());
		}
	}
#endif // ENABLE_DRAW_DEBUG
}


void AActor::InvalidateLightingCacheDetailed(bool bTranslationOnly)
{
	if(GIsEditor && !GIsDemoMode)
	{
		for (UActorComponent* Component : GetComponents())
		{
			if (Component && Component->IsRegistered())
			{
				Component->InvalidateLightingCacheDetailed(true, bTranslationOnly);
			}
		}
	}
}

 // COLLISION

bool AActor::ActorLineTraceSingle(struct FHitResult& OutHit, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params) const
{
	OutHit = FHitResult(1.f);
	OutHit.TraceStart = Start;
	OutHit.TraceEnd = End;
	bool bHasHit = false;
	
	for (UActorComponent* Component : GetComponents())
	{
		FHitResult HitResult;
		UPrimitiveComponent* Primitive = Cast<UPrimitiveComponent>(Component);
		if (Primitive && Primitive->IsRegistered() && Primitive->IsCollisionEnabled() 
			&& (Primitive->GetCollisionResponseToChannel(TraceChannel) == ECollisionResponse::ECR_Block) 
			&& Primitive->LineTraceComponent(HitResult, Start, End, Params) )
		{
			// return closest hit
			if( HitResult.Time < OutHit.Time )
			{
				OutHit = HitResult;
				bHasHit = true;
			}
		}
	}

	return bHasHit;
}

float AActor::ActorGetDistanceToCollision(const FVector& Point, ECollisionChannel TraceChannel, FVector& ClosestPointOnCollision, UPrimitiveComponent** OutPrimitiveComponent) const
{
	ClosestPointOnCollision = Point;
	float ClosestPointDistanceSqr = -1.f;

	TInlineComponentArray<UPrimitiveComponent*> Components;
	GetComponents(Components);

	for (int32 ComponentIndex=0; ComponentIndex<Components.Num(); ComponentIndex++)
	{
		UPrimitiveComponent* Primitive = Components[ComponentIndex];
		if( Primitive->IsRegistered() && Primitive->IsCollisionEnabled() 
			&& (Primitive->GetCollisionResponseToChannel(TraceChannel) == ECollisionResponse::ECR_Block) )
		{
			FVector ClosestPoint;
			float DistanceSqr = -1.f;

			if (!Primitive->GetSquaredDistanceToCollision(Point, DistanceSqr, ClosestPoint))
			{
				// Invalid result, impossible to be better than ClosestPointDistance
				continue;
			}

			if( (ClosestPointDistanceSqr < 0.f) || (DistanceSqr < ClosestPointDistanceSqr) )
			{
				ClosestPointDistanceSqr = DistanceSqr;
				ClosestPointOnCollision = ClosestPoint;
				if( OutPrimitiveComponent )
				{
					*OutPrimitiveComponent = Primitive;
				}

				// If we're inside collision, we're not going to find anything better, so abort search we've got our best find.
				if( DistanceSqr <= UE_KINDA_SMALL_NUMBER )
				{
					break;
				}
			}
		}
	}

	return (ClosestPointDistanceSqr > 0.f ? FMath::Sqrt(ClosestPointDistanceSqr) : ClosestPointDistanceSqr);
}


void AActor::LifeSpanExpired()
{
	Destroy();
}

void AActor::SetLifeSpan( float InLifespan )
{
	// Store the new value
	InitialLifeSpan = InLifespan;
	// Initialize a timer for the actors lifespan if there is one. Otherwise clear any existing timer
	if ((GetLocalRole() == ROLE_Authority || GetTearOff()) && IsValidChecked(this) && GetWorld())
	{
		if( InLifespan > 0.0f)
		{
			// Do not start the lifespan timer if the level is already being removed from the world, 
			// as this could cause UActorComponent::DestroyPhysicsState to be called while the component 
			// is still in the pre-unregistering state (UActorComponent::IsPreUnregistering == true).
			// This should not cause any issues since the actor is about to be destroyed.
			if (GetLevel() && GetLevel()->bIsBeingRemoved)
			{
				UE_LOGF(LogActor, Log, "AActor::SetLifeSpan call on Actor %ls was ignored (level is already being removed).", *GetName());
			}
			else
			{
				GetWorldTimerManager().SetTimer(TimerHandle_LifeSpanExpired, this, &AActor::LifeSpanExpired, InLifespan);
			}
		}
		else
		{
			GetWorldTimerManager().ClearTimer( TimerHandle_LifeSpanExpired );		
		}
	}
}

float AActor::GetLifeSpan() const
{
	if (UWorld* World = GetWorld())
	{
		// Timer remaining returns -1.0f if there is no such timer - return this as ZERO
		const float CurrentLifespan = World->GetTimerManager().GetTimerRemaining(TimerHandle_LifeSpanExpired);
		return (CurrentLifespan != -1.0f) ? CurrentLifespan : 0.0f;
	}
	
	return 0.0f;
}

void AActor::PostInitializeComponents()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Actor_PostInitComponents);

	if(IsValidChecked(this) )
	{
		bActorInitialized = true;
		
		UpdateAllReplicatedComponents();
	}
}

void AActor::PreInitializeComponents()
{
	if (AutoReceiveInput != EAutoReceiveInput::Disabled)
	{
		const int32 PlayerIndex = int32(AutoReceiveInput.GetValue()) - 1;

		APlayerController* PC = UGameplayStatics::GetPlayerController(this, PlayerIndex);
		if (PC)
		{
			EnableInput(PC);
		}
		else
		{
			GetWorld()->PersistentLevel->RegisterActorForAutoReceiveInput(this, PlayerIndex);
		}
	}
}

float AActor::GetActorTimeDilation() const
{
	// get actor custom time dilation
	// if you do slomo, that changes WorldSettings->TimeDilation
	// So multiply to get final TimeDilation
	return CustomTimeDilation * GetWorldSettings()->GetEffectiveTimeDilation();
}

float AActor::GetActorTimeDilation(const UWorld& ActorWorld) const
{
	checkSlow(&ActorWorld == GetWorld());
	return CustomTimeDilation * ActorWorld.GetWorldSettings()->GetEffectiveTimeDilation();
}

float AActor::GetDistanceTo(const AActor* OtherActor) const
{
	return OtherActor ? (GetActorLocation() - OtherActor->GetActorLocation()).Size() : 0.f;
}

float AActor::GetSquaredDistanceTo(const AActor* OtherActor) const
{
	return OtherActor ? (GetActorLocation() - OtherActor->GetActorLocation()).SizeSquared() : 0.f;
}

float AActor::GetHorizontalDistanceTo(const AActor* OtherActor) const
{
	return OtherActor ? (GetActorLocation() - OtherActor->GetActorLocation()).Size2D() : 0.f;
}

float AActor::GetSquaredHorizontalDistanceTo(const AActor* OtherActor) const
{
	return OtherActor ? (GetActorLocation() - OtherActor->GetActorLocation()).SizeSquared2D() : 0.f;
}

float AActor::GetVerticalDistanceTo(const AActor* OtherActor) const
{
	return OtherActor ? FMath::Abs((GetActorLocation().Z - OtherActor->GetActorLocation().Z)) : 0.f;
}

float AActor::GetDotProductTo(const AActor* OtherActor) const
{
	if (OtherActor)
	{
		FVector Dir = GetActorForwardVector();
		FVector Offset = OtherActor->GetActorLocation() - GetActorLocation();
		Offset = Offset.GetSafeNormal();
		return FVector::DotProduct(Dir, Offset);
	}
	return -2.0;
}

float AActor::GetHorizontalDotProductTo(const AActor* OtherActor) const
{
	if (OtherActor)
	{
		FVector Dir = GetActorForwardVector();
		FVector Offset = OtherActor->GetActorLocation() - GetActorLocation();
		Offset = Offset.GetSafeNormal2D();
		return FVector::DotProduct(Dir, Offset);
	}
	return -2.0;
}


#if WITH_EDITOR
const int32 AActor::GetNumUncachedStaticLightingInteractions() const
{
	if (GetRootComponent())
	{
		return GetRootComponent()->GetNumUncachedStaticLightingInteractions();
	}
	return 0;
}
#endif // WITH_EDITOR


// K2 versions of various transform changing operations.
// Note: we pass null for the hit result if not sweeping, for better perf.
// This assumes this K2 function is only used by blueprints, which initializes the param for each function call.

bool AActor::K2_SetActorLocation(FVector NewLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	return SetActorLocation(NewLocation, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

bool AActor::K2_SetActorRotation(FRotator NewRotation, bool bTeleportPhysics)
{
	return SetActorRotation(NewRotation, TeleportFlagToEnum(bTeleportPhysics));
}

bool AActor::K2_SetActorLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	return SetActorLocationAndRotation(NewLocation, NewRotation, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_AddActorWorldOffset(FVector DeltaLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	AddActorWorldOffset(DeltaLocation, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_AddActorWorldRotation(FRotator DeltaRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	AddActorWorldRotation(DeltaRotation, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_AddActorWorldTransform(const FTransform& DeltaTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	AddActorWorldTransform(DeltaTransform, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

 void AActor::K2_AddActorWorldTransformKeepScale(const FTransform& DeltaTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
 {
 	AddActorWorldTransformKeepScale(DeltaTransform, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
 }

bool AActor::K2_SetActorTransform(const FTransform& NewTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	return SetActorTransform(NewTransform, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_AddActorLocalOffset(FVector DeltaLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	AddActorLocalOffset(DeltaLocation, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_AddActorLocalRotation(FRotator DeltaRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	AddActorLocalRotation(DeltaRotation, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_AddActorLocalTransform(const FTransform& NewTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	AddActorLocalTransform(NewTransform, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_SetActorRelativeLocation(FVector NewRelativeLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	SetActorRelativeLocation(NewRelativeLocation, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_SetActorRelativeRotation(FRotator NewRelativeRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	SetActorRelativeRotation(NewRelativeRotation, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

void AActor::K2_SetActorRelativeTransform(const FTransform& NewRelativeTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport)
{
	SetActorRelativeTransform(NewRelativeTransform, bSweep, (bSweep ? &SweepHitResult : nullptr), TeleportFlagToEnum(bTeleport));
}

float AActor::GetGameTimeSinceCreation() const
{
	if (UWorld* MyWorld = GetWorld())
	{
		return MyWorld->GetTimeSeconds() - CreationTime;		
	}
	// return 0.f if GetWorld return's null
	else
	{
		return 0.f;
	}
}

void AActor::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	if (UWorld* MyWorld = GetWorld())
	{
		if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(MyWorld))
		{
			for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
			{
				if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(this))
				{
					Driver.NetDriver->NotifyActorRenamed(this, OldOuter, OldName);
				}
			}
		}
	}
}

bool AActor::IsHLODRelevant() const
{
	if (!IsValidChecked(this))
	{
		return false;
	}

	if (HasAnyFlags(RF_Transient))
	{
#if WITH_EDITOR
		// Treat actors in LI as HLOD relevant for the sake of visualisation modes
		if (!IsInLevelInstance())
		{
			return false;
		}
		else if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(GetWorld()))
		{
			if (const ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetOwningLevelInstance(GetLevel()))
			{
				if (!LevelInstance->IsLevelInstanceHLODRelevant())
				{
					return false;
				}
			}
		}
#else
		return false;
#endif
	}

	if (IsHidden())
	{
		return false;
	}

	if (IsEditorOnly())
	{
		return false;
	}

	if (!bEnableAutoLODGeneration)
	{
		return false;
	}

#if WITH_EDITOR 
	// Only spatially loaded actors can be HLOD relevant in partitioned worlds
	if (UWorld::IsPartitionedWorld(GetWorld()) && !GetIsSpatiallyLoaded())
	{
		return false;
	}
#endif

	return HasHLODRelevantComponents();
}

bool AActor::HasHLODRelevantComponents() const
{
	return Algo::AnyOf(GetComponents(), [](const UActorComponent* Component) { return Component && Component->IsHLODRelevant(); });
}

TArray<UActorComponent*> AActor::GetHLODRelevantComponents() const
{
	TArray<UActorComponent*> HLODRelevantComponents;
	auto IsHLODRelevant = [](UActorComponent* Component) { return Component && Component->IsHLODRelevant(); };
	auto GetComponent = [](UActorComponent* Component) { return Component; };
	Algo::TransformIf(GetComponents(), HLODRelevantComponents, IsHLODRelevant, GetComponent);
	return HLODRelevantComponents;
}

void AActor::SetLODParent(UPrimitiveComponent* InLODParent, float InParentDrawDistance)
{
	if (InLODParent && InLODParent->MinDrawDistance != InParentDrawDistance)
	{
		InLODParent->MinDrawDistance = InParentDrawDistance;
		InLODParent->MarkRenderStateDirty();
	}

	for (UActorComponent* Component : GetComponents())
	{
		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
		{
			// parent primitive will be null if no LOD parent is selected
			PrimitiveComponent->SetLODParentPrimitive(InLODParent);
		}
	}
}

void AActor::SetRayTracingGroupId(int32 InRaytracingGroupId)
{
	if (RayTracingGroupId != InRaytracingGroupId)
	{
		Modify();
		RayTracingGroupId = InRaytracingGroupId;
		MarkComponentsRenderStateDirty();
	}
}

int32 AActor::GetRayTracingGroupId() const
{
	// Support for ChildActorComponents 
	if(RayTracingGroupId == FPrimitiveSceneProxy::InvalidRayTracingGroupId && GetParentActor() != nullptr)
	{
		return GetParentActor()->GetRayTracingGroupId();
	}
	
	return RayTracingGroupId;
}

void AActor::SetHidden(bool bInHidden)
{
	bHidden = bInHidden;
	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, bHidden, this);
}

void AActor::SetReplicatingMovement(bool bInReplicateMovement)
{
	bReplicateMovement = bInReplicateMovement;
	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, bReplicateMovement, this);
}

void AActor::SetCanBeDamaged(bool bInCanBeDamaged)
{
	bCanBeDamaged = bInCanBeDamaged;
	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, bCanBeDamaged, this);
}

void AActor::SetRole(ENetRole InRole)
{
	Role = InRole;
	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, Role, this);
}

FRepMovement& AActor::GetReplicatedMovement_Mutable()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, ReplicatedMovement, this);
	return ReplicatedMovement;
}

void AActor::SetReplicatedMovement(const FRepMovement& InReplicatedMovement)
{
	const bool bRepPhysicsDiffers = (GetReplicatedMovement().bRepPhysics != InReplicatedMovement.bRepPhysics);
	
	GetReplicatedMovement_Mutable() = InReplicatedMovement;

	if (bRepPhysicsDiffers)
	{
		UpdateReplicatePhysicsCondition();
	}
}

void AActor::SetNetUpdateFrequency(float Frequency)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NetUpdateFrequency = Frequency;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_LOGF(LogNet, Verbose, "AActor::SetNetUpdateFrequency(): %ls Frequency=%f HasActorBegunPlay()=%d", *this->GetFullName(), Frequency, HasActorBegunPlay());

	if (IsActorInitialized())
	{
		ActorUtils::ForEachNetDriver(GEngine, GetWorld(), [this](UNetDriver* NetDriver)
		{
			if (NetDriver)
			{
				NetDriver->GetOnNetUpdateFrequencyChanged().Broadcast(this);
			}
		});
	}
}

float AActor::GetNetUpdateFrequency() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return NetUpdateFrequency;
PRAGMA_ENABLE_DEPRECATION_WARNINGS 
}

void AActor::SetMinNetUpdateFrequency(float MinFrequency)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS 
	MinNetUpdateFrequency = MinFrequency;
PRAGMA_ENABLE_DEPRECATION_WARNINGS 
}

float AActor::GetMinNetUpdateFrequency() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS 
	return MinNetUpdateFrequency;
PRAGMA_ENABLE_DEPRECATION_WARNINGS 
}

void AActor::SetNetCullDistanceSquared(float DistanceSq)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS 
	NetCullDistanceSquared = DistanceSq;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

float AActor::GetNetCullDistanceSquared() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return NetCullDistanceSquared;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void AActor::SetInstigator(APawn* InInstigator)
{
	Instigator = InInstigator;
	MARK_PROPERTY_DIRTY_FROM_NAME(AActor, Instigator, this);
}

#if WITH_EDITOR
TFunction<bool(const AActor*)> GIsActorSelectedInEditor;
#endif

FArchive& operator<<(FArchive& Ar, FActorRootComponentReconstructionData::FAttachedActorInfo& ActorInfo)
{
	enum class EVersion : uint8
	{
		InitialVersion = 0,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	EVersion Version = EVersion::LatestVersion;
	Ar << Version;

	if (Version > EVersion::LatestVersion)
	{
		Ar.SetError();
		return Ar;
	}

	Ar << ActorInfo.Actor;
	Ar << ActorInfo.AttachParent;
	Ar << ActorInfo.AttachParentName;
	Ar << ActorInfo.SocketName;
	Ar << ActorInfo.RelativeTransform;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FActorRootComponentReconstructionData& RootComponentData)
{
	enum class EVersion : uint8
	{
		InitialVersion = 0,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	EVersion Version = EVersion::LatestVersion;
	Ar << Version;

	if (Version > EVersion::LatestVersion)
	{
		Ar.SetError();
		return Ar;
	}

	Ar << RootComponentData.Transform;

	if (Ar.IsSaving())
	{
		FQuat TransformRotationQuat = RootComponentData.TransformRotationCache.GetCachedQuat();
		Ar << TransformRotationQuat;
	}
	else if (Ar.IsLoading())
	{
		FQuat TransformRotationQuat;
		Ar << TransformRotationQuat;
		RootComponentData.TransformRotationCache.NormalizedQuatToRotator(TransformRotationQuat);
	}

	Ar << RootComponentData.AttachedParentInfo;

	Ar << RootComponentData.AttachedToInfo;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FActorTransactionAnnotationData& ActorTransactionAnnotationData)
{
	enum class EVersion : uint8
	{
		InitialVersion = 0,
		WithInstanceCache,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	EVersion Version = EVersion::LatestVersion;
	Ar << Version;

	if (Version > EVersion::LatestVersion)
	{
		Ar.SetError();
		return Ar;
	}

	// InitialVersion
	Ar << ActorTransactionAnnotationData.Actor;
	Ar << ActorTransactionAnnotationData.bRootComponentDataCached;
	if (ActorTransactionAnnotationData.bRootComponentDataCached)
	{
		Ar << ActorTransactionAnnotationData.RootComponentData;
	}
	// WithInstanceCache
	if (Ar.IsLoading())
	{
		ActorTransactionAnnotationData.ComponentInstanceData = FComponentInstanceDataCache(ActorTransactionAnnotationData.Actor.Get());
	}
	if (Version >= EVersion::WithInstanceCache)
	{
		ActorTransactionAnnotationData.ComponentInstanceData.Serialize(Ar);
	}

	return Ar;
}

//---------------------------------------------------------------------------
// DataLayers (begin)

TArray<const UDataLayerInstance*> AActor::GetDataLayerInstances() const
{
	const bool bUseLevelContext = false;
	return GetDataLayerInstancesInternal(bUseLevelContext);
}

// Returns all valid DataLayerInstances for this actor including those inherited from their parent level instance actor.
// If bUseLevelContext is true, the actor level will be used to find the associated DataLayerManager which will be used 
// to resolve valid datalayers for this particular level.
TArray<const UDataLayerInstance*> AActor::GetDataLayerInstancesInternal(bool bUseLevelContext, bool bIncludeParentDataLayers) const
{
	if (const IWorldPartitionCell* Cell = GetWorldPartitionRuntimeCell())
	{
		return Cell->GetDataLayerInstances();
	}

#if WITH_EDITOR
	{
		TArray<const UDataLayerInstance*> DataLayerInstances;
		if (UDataLayerManager* DataLayerManager = bUseLevelContext ? UDataLayerManager::GetDataLayerManager(this) : UDataLayerManager::GetDataLayerManager(GetWorld()))
		{
			DataLayerInstances += DataLayerManager->GetDataLayerInstances(GetDataLayerAssets());
		}

		if (bIncludeParentDataLayers)
		{
			// Add parent container Data Layer Instances
			ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(GetWorld());
			if (AActor* ParentActor = LevelInstanceSubsystem ? Cast<AActor>(LevelInstanceSubsystem->GetParentLevelInstance(this)) : nullptr)
			{
				for (const UDataLayerInstance* DataLayerInstance : ParentActor->GetDataLayerInstancesInternal(bUseLevelContext, bIncludeParentDataLayers))
				{
					DataLayerInstances.AddUnique(DataLayerInstance);
				}
			}
		}
		return DataLayerInstances;
	}
#else
	return TArray<const UDataLayerInstance*>();
#endif
}

bool AActor::ContainsDataLayer(const UDataLayerInstance* DataLayerInstance) const
{
	if (const IWorldPartitionCell* Cell = GetWorldPartitionRuntimeCell())
	{
		return Cell->ContainsDataLayer(DataLayerInstance);
	}

#if WITH_EDITOR
	return GetDataLayerInstances().Contains(DataLayerInstance);
#else
	return false;
#endif
}

bool AActor::ContainsDataLayer(const UDataLayerAsset* DataLayerAsset) const
{
	if (!DataLayerAsset)
	{
		return false;
	}

	if (const IWorldPartitionCell* Cell = GetWorldPartitionRuntimeCell())
	{
		return Cell->ContainsDataLayer(DataLayerAsset);
	}

#if WITH_EDITOR
	if (DataLayerAssets.Contains(DataLayerAsset) || (ExternalDataLayerAsset == DataLayerAsset))
	{
		return true;
	}
	// Add parent container Data Layer Instances
	ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(GetWorld());
	if (AActor* ParentActor = LevelInstanceSubsystem ? Cast<AActor>(LevelInstanceSubsystem->GetParentLevelInstance(this)) : nullptr)
	{
		return ParentActor->ContainsDataLayer(DataLayerAsset);
	}
#endif
	return false;
}

const UExternalDataLayerAsset* AActor::GetExternalDataLayerAsset() const
{
	if (const IWorldPartitionCell* Cell = GetWorldPartitionRuntimeCell())
	{
		const UExternalDataLayerInstance* ExternalDataLayerInstance = Cell->GetExternalDataLayerInstance();
		return ExternalDataLayerInstance ? ExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr;
	}
#if WITH_EDITOR
	return ExternalDataLayerAsset;
#else
	return nullptr;
#endif
}

bool AActor::HasDataLayers() const
{
	if (const IWorldPartitionCell* Cell = GetWorldPartitionRuntimeCell())
	{
		return Cell->GetDataLayers().Num() > 0;
	}

#if WITH_EDITOR
	return (GetDataLayerInstances().Num() > 0);
#else
	return false;
#endif
}

bool AActor::HasContentBundle() const
{
	if (const IWorldPartitionCell* Cell = GetWorldPartitionRuntimeCell())
	{
		return Cell->HasContentBundle();
	}

#if WITH_EDITOR
	return GetContentBundleGuid().IsValid();
#else
	return false;
#endif
}

const IWorldPartitionCell* AActor::GetWorldPartitionRuntimeCell() const
{
	if (ULevel* Level = GetLevel())
	{
		return Level->GetWorldPartitionRuntimeCell();
	}
	return nullptr;
}


// DataLayers (end)
//---------------------------------------------------------------------------


void AActor::GetActorClassDefaultComponents(const TSubclassOf<AActor>& InActorClass, const TSubclassOf<UActorComponent>& InComponentClass, TArray<const UActorComponent*>& OutComponents)
{
	OutComponents.Reset();

	ForEachComponentOfActorClassDefault(InActorClass, InComponentClass, [&](const UActorComponent* TemplateComponent)
	{
		OutComponents.Add(TemplateComponent);
		return true;
	});
}

const UActorComponent* AActor::GetActorClassDefaultComponentByName(const TSubclassOf<AActor>& InActorClass, const TSubclassOf<UActorComponent>& InComponentClass, FName InComponentName)
{
	const UActorComponent* Result = nullptr;

	ForEachComponentOfActorClassDefault(InActorClass, InComponentClass, [&Result, InComponentName](const UActorComponent* TemplateComponent)
	{
		// Try to strip suffix used to identify template component instances
		FString StrippedName = TemplateComponent->GetName();
		if (StrippedName.RemoveFromEnd(UActorComponent::ComponentTemplateNameSuffix))
		{
			if (StrippedName == InComponentName.ToString())
			{
				Result = TemplateComponent;
				return false;
			}
		}
		else if (TemplateComponent->GetFName() == InComponentName)
		{
			Result = TemplateComponent;
			return false;
		}

		return true;
	});

	return Result;
}

const UActorComponent* AActor::GetActorClassDefaultComponent(const TSubclassOf<AActor>& InActorClass, const TSubclassOf<UActorComponent>& InComponentClass)
{
	const UActorComponent* Result = nullptr;

	ForEachComponentOfActorClassDefault(InActorClass, InComponentClass, [&Result](const UActorComponent* TemplateComponent)
	{
		Result = TemplateComponent;
		return false;
	});

	return Result;
}

void AActor::ForEachComponentOfActorClassDefault(const TSubclassOf<AActor>& ActorClass, const TSubclassOf<UActorComponent>& InComponentClass, TFunctionRef<bool(const UActorComponent*)> InFunc)
{
	if (!ActorClass.Get())
	{
		return;
	}

	auto FilterFunc = [&](const UActorComponent* TemplateComponent)
	{
		if (!TemplateComponent)
		{
			return true;
		}
		
		if (!InComponentClass.Get() || TemplateComponent->IsA(InComponentClass))
		{
			return InFunc(TemplateComponent);
		}

		return true;
	};

	// Process native components
	const AActor* CDO = ActorClass->GetDefaultObject<AActor>();
	for (const UActorComponent* Component : CDO->GetComponents())
	{
		if (!FilterFunc(Component))
		{
			return;
		}
	}

	// Process blueprint components
	if (UBlueprintGeneratedClass* ActorBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(ActorClass))
	{
		UBlueprintGeneratedClass::ForEachGeneratedClassInHierarchy(ActorClass, [&](const UBlueprintGeneratedClass* CurrentBPGC)
		{
			if (const USimpleConstructionScript* const ConstructionScript = CurrentBPGC->SimpleConstructionScript)
			{
				// Gets all BP added components
				for (const USCS_Node* const Node : ConstructionScript->GetAllNodes())
				{
					if (!FilterFunc(Node->GetActualComponentTemplate(ActorBlueprintGeneratedClass)))
					{
						return false;
					}
				}
			}
			return true;
		});
	}
}

#undef LOCTEXT_NAMESPACE
