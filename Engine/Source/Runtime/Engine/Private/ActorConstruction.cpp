// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineLogs.h"
#include "Math/RandomStream.h"
#include "Misc/ScopeExit.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UObject/UObjectThreadContext.h"
#include "Serialization/ObjectReader.h"
#include "Engine/Blueprint.h"
#include "ActorTransactionAnnotation.h"
#include "Components/StaticMeshComponent.h"
#include "Components/BillboardComponent.h"
#include "Misc/ConfigCacheIni.h"
#include "Engine/Texture2D.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/CullDistanceVolume.h"
#include "Engine/SimpleConstructionScript.h"
#include "ScopedSuspendRerunConstructionScripts.h"

#if WITH_EDITOR
#include "Cooker/CookRand.h"
#include "Editor.h"
#else
#include "Engine/World.h"
#include "UObject/Package.h"
#endif

DEFINE_LOG_CATEGORY(LogBlueprintUserMessages);

DECLARE_CYCLE_STAT(TEXT("InstanceActorComponent"), STAT_InstanceActorComponent, STATGROUP_Engine);

//////////////////////////////////////////////////////////////////////////
// AActor Blueprint Stuff

#if WITH_EDITOR
void AActor::SeedAllRandomStreams()
{
	UScriptStruct* RandomStreamStruct = TBaseStructure<FRandomStream>::Get();
	UObject* Archetype = GetArchetype();

	for (TFieldIterator<FStructProperty> It(GetClass()); It; ++It)
	{
		FStructProperty* StructProp = *It;
		if (StructProp->Struct == RandomStreamStruct)
		{
			FRandomStream* ArchetypeStreamPtr = StructProp->ContainerPtrToValuePtr<FRandomStream>(Archetype);
			FRandomStream* StreamPtr = StructProp->ContainerPtrToValuePtr<FRandomStream>(this);

			if (ArchetypeStreamPtr->GetInitialSeed() == 0)
			{
				StreamPtr->GenerateNewSeed();
			}
			else
			{
				StreamPtr->Reset();
			}
		}
	}
}
#endif //WITH_EDITOR

bool IsBlueprintAddedContainer(FProperty* Prop)
{
	if (Prop->IsA<FArrayProperty>() || Prop->IsA<FSetProperty>() || Prop->IsA<FMapProperty>())
	{
		return Prop->GetOwnerUObject()->IsInBlueprint();
	}

	return false;
}

void AActor::ResetPropertiesForConstruction()
{
	// Get class CDO
	AActor* Default = GetClass()->GetDefaultObject<AActor>();
	// RandomStream struct name to compare against
	const FName RandomStreamName("RandomStream");

	// We don't want to reset references to world object
	UWorld* World = GetWorld();
	const bool bIsLevelScriptActor = IsA<ALevelScriptActor>();
	const bool bIsPlayInEditor = World && World->IsPlayInEditor();

	// Iterate over properties
	for( TFieldIterator<FProperty> It(GetClass()) ; It ; ++It )
	{
		FProperty* Prop = *It;
		FStructProperty* StructProp = CastField<FStructProperty>(Prop);
		UClass* PropClass = Prop->GetOwnerChecked<UClass>(); // get the class that added this property

		// First see if it is a random stream, if so reset before running construction script
		if( (StructProp != nullptr) && (StructProp->Struct != nullptr) && (StructProp->Struct->GetFName() == RandomStreamName) )
		{
			FRandomStream* StreamPtr =  StructProp->ContainerPtrToValuePtr<FRandomStream>(this);
			StreamPtr->Reset();
		}
		// If it is a blueprint exposed variable that is not editable per-instance, reset to default before running construction script
		else if (!bIsLevelScriptActor && (!Prop->ContainsInstancedObjectProperty() || IsBlueprintAddedContainer(Prop)))
		{
			const bool bExposedOnSpawn = bIsPlayInEditor && Prop->HasAnyPropertyFlags(CPF_ExposeOnSpawn);
			const bool bCanEditInstanceValue = !Prop->HasAnyPropertyFlags(CPF_DisableEditOnInstance) && Prop->HasAnyPropertyFlags(CPF_Edit);
			const bool bCanBeSetInBlueprints = Prop->HasAnyPropertyFlags(CPF_BlueprintVisible) && !Prop->HasAnyPropertyFlags(CPF_BlueprintReadOnly);

			if (!bExposedOnSpawn
				&& !bCanEditInstanceValue
				&& bCanBeSetInBlueprints
				&& !Prop->IsA<FDelegateProperty>()
				&& !Prop->IsA<FMulticastDelegateProperty>())
			{
				Prop->CopyCompleteValue_InContainer(this, Default);
			}
		}
	}
}

 int32 CalcComponentAttachDepth(UActorComponent* InComp, TMap<UActorComponent*, int32>& ComponentDepthMap) 
 {
	int32 ComponentDepth = 0;
	int32* CachedComponentDepth = ComponentDepthMap.Find(InComp);
	if (CachedComponentDepth)
	{
		ComponentDepth = *CachedComponentDepth;
	}
	else
	{
		if (USceneComponent* SC = Cast<USceneComponent>(InComp))
		{
			if (SC->GetAttachParent() && SC->GetAttachParent()->GetOwner() == InComp->GetOwner())
			{
				ComponentDepth = CalcComponentAttachDepth(SC->GetAttachParent(), ComponentDepthMap) + 1;
			}
		}
		ComponentDepthMap.Add(InComp, ComponentDepth);
	}

	return ComponentDepth;
 }

//* Destroys the constructed components.
void AActor::DestroyConstructedComponents()
{
	// Remove all existing components
	TInlineComponentArray<UActorComponent*> PreviouslyAttachedComponents;
	GetComponents(PreviouslyAttachedComponents);

	TMap<UActorComponent*, int32> ComponentDepthMap;

	for (UActorComponent* Component : PreviouslyAttachedComponents)
	{
		if (Component)
		{
			CalcComponentAttachDepth(Component, ComponentDepthMap);
		}
	}

	ComponentDepthMap.ValueSort([](const int32& A, const int32& B)
	{
		return A > B;
	});

	bool bMarkPackageDirty = false;
	for (const TPair<UActorComponent*,int32>& ComponentAndDepth : ComponentDepthMap)
	{
		UActorComponent* Component = ComponentAndDepth.Key;
		if (Component)
		{
			bool bDestroyComponent = false;
			if (Component->IsCreatedByConstructionScript())
			{
				bDestroyComponent = true;
			}
			else
			{
				UActorComponent* OuterComponent = Component->GetTypedOuter<UActorComponent>();
				while (OuterComponent)
				{
					if (OuterComponent->IsCreatedByConstructionScript())
					{
						bDestroyComponent = true;
						break;
					}
					OuterComponent = OuterComponent->GetTypedOuter<UActorComponent>();
				}
			}

			if (!bDestroyComponent)
			{
				// check for orphaned natively created components:
				if (Component->CreationMethod == EComponentCreationMethod::Native && Component->HasAnyFlags(RF_DefaultSubObject))
				{
					UObject* ComponentArchetype = Component->GetArchetype();
					if (ComponentArchetype == ComponentArchetype->GetClass()->GetDefaultObject(false))
					{
						bDestroyComponent = true;
					}
				}
			}

			if (bDestroyComponent)
			{
				if (Component == RootComponent)
				{
					RootComponent = NULL;
				}

				Component->DestroyComponent();

				// Rename component to avoid naming conflicts in the case where we rerun the SCS and name the new components the same way.
				FName const NewBaseName( *(FString::Printf(TEXT("TRASH_%s"), *Component->GetClass()->GetName())) );
				FName const NewObjectName = MakeUniqueObjectName(GetTransientOuterForRename(Component->GetClass()), GetClass(), NewBaseName);
				Component->Rename(*NewObjectName.ToString(), GetTransientOuterForRename(Component->GetClass()), REN_DontCreateRedirectors|REN_NonTransactional|REN_DoNotDirty);


				// Transient actors should never mark the package as dirty
				bMarkPackageDirty = !HasAnyFlags(RF_Transient);
			}
		}
	}

	if (bMarkPackageDirty)
	{
		GetPackage()->MarkPackageDirty();
	}

	// When a constructed component is destroyed, it is removed from this set. We rebuild it from the original set to compact
	// the set and ensure that the iteration order remains stable after re-running actor construction scripts.
	if (PreviouslyAttachedComponents.Num() > OwnedComponents.Num())
	{
		TSet<TObjectPtr<UActorComponent>> OldComponents = MoveTemp(OwnedComponents);
		OwnedComponents.Reserve(OldComponents.Num());
		for (UActorComponent* Component : PreviouslyAttachedComponents)
		{
			if (OldComponents.Contains(Component))
			{
				OwnedComponents.Add(Component);
			}
		}
	}
}


bool AActor::HasNonTrivialUserConstructionScript() const
{
	UFunction* UCS = GetClass()->FindFunctionByName(FName("UserConstructionScript")/*UEdGraphSchema_K2::FN_UserConstructionScript*/);
	if (UCS && UCS->Script.Num())
	{
		return true;
	}
	return false;
}

#if WITH_EDITOR
void AActor::RerunConstructionScripts()
{
	if (FScopedSuspendRerunConstructionScripts::IsSuspended())
	{
		// Queue this actor for a deferred rerun
		FScopedSuspendRerunConstructionScripts::DeferRerun(this);
		return;
	}

	checkf(!HasAnyFlags(RF_ClassDefaultObject), TEXT("RerunConstructionScripts should never be called on a CDO as it can mutate the transient data on the CDO which then propagates to instances!"));

	FEditorScriptExecutionGuard ScriptGuard;
	// don't allow (re)running construction scripts on dying actors and Actors that seamless traveled 
	// were constructed in the previous level and should not have construction scripts rerun
	bool bAllowReconstruction = !bActorSeamlessTraveled && IsValidChecked(this) && !HasAnyFlags(RF_BeginDestroyed|RF_FinishDestroyed);
	if (bAllowReconstruction)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AActor::RerunConstructionScripts);
		if(GIsEditor)
		{
			// Don't allow reconstruction if we're still in the middle of construction.
			bAllowReconstruction = !bActorIsBeingConstructed;
			if (ensureMsgf(bAllowReconstruction, TEXT("Attempted to rerun construction scripts on an Actor that isn't fully constructed yet (%s)."), *GetFullName()))
			{
				// Generate the blueprint hierarchy for this actor
				TArray<UBlueprint*> ParentBPStack;
				bAllowReconstruction = UBlueprint::GetBlueprintHierarchyFromClass(GetClass(), ParentBPStack);
				if (bAllowReconstruction)
				{
					for (int i = ParentBPStack.Num() - 1; i > 0 && bAllowReconstruction; --i)
					{
						const UBlueprint* ParentBP = ParentBPStack[i];
						if (ParentBP && ParentBP->bBeingCompiled)
						{
							// don't allow (re)running construction scripts if a parent BP is being compiled
							bAllowReconstruction = false;
						}
					}
				}
			}
		}

		// Child Actors can be customized in many ways by their parents construction scripts and rerunning directly on them would wipe
		// that out. So instead we redirect up the hierarchy
		if (IsChildActor())
		{
			if (AActor* ParentActor = GetParentComponent()->GetOwner())
			{
				ParentActor->RerunConstructionScripts();
				return;
			}
		}

		// Set global flag to let system know we are reconstructing blueprint instances
		TGuardValue<bool> GuardTemplateNameFlag(GIsReconstructingBlueprintInstances, true);

		// Temporarily suspend the undo buffer; we don't need to record reconstructed component objects into the current transaction
		ITransaction* CurrentTransaction = GUndo;
		GUndo = nullptr;

		// Keep track of non-dirty packages, so we can clear the dirty state after reconstruction.
		TSet<UPackage*> CleanPackageList;
		auto CheckAndSaveOuterPackageToCleanList = [&CleanPackageList](const UObject* InObject)
		{
			check(InObject);
			UPackage* ObjectPackage = InObject->GetPackage();
			if (ObjectPackage && !ObjectPackage->IsDirty() && ObjectPackage != GetTransientPackage())
			{
				CleanPackageList.Add(ObjectPackage);
			}
		};

		// Mark package as clean on exit if not transient and not already dirty.
		CheckAndSaveOuterPackageToCleanList(this);
		
		// Create cache to store component data across rerunning construction scripts
		FComponentInstanceDataCache* InstanceDataCache;
		
		FTransform OldTransform = FTransform::Identity;
		FRotationConversionCache OldTransformRotationCache; // Enforces using the same Rotator.
		FName  SocketName;
		AActor* Parent = nullptr;
		USceneComponent* AttachParentComponent = nullptr;

		bool bUseRootComponentProperties = true;

		// Struct to store info about attached actors
		struct FAttachedActorInfo
		{
			AActor* AttachedActor;
			FName AttachedToSocket;
			bool bSetRelativeTransform;
			FTransform RelativeTransform;
			FName AttachParentName;
		};

		// Save info about attached actors
		TArray<FAttachedActorInfo> AttachedActorInfos;

		// Before we build the component instance data cache we need to make sure that instance components 
		// are correctly in their AttachParent's AttachChildren array which may not be the case if they
		// have not yet been registered
		for (UActorComponent* Component : InstanceComponents)
		{
			if (Component && !Component->IsRegistered())
			{
				if (USceneComponent* SceneComp = Cast<USceneComponent>(Component))
				{
					if (USceneComponent* AttachParent = SceneComp->GetAttachParent())
					{
						if (AttachParent->IsCreatedByConstructionScript())
						{
							SceneComp->AttachToComponent(AttachParent, FAttachmentTransformRules::KeepRelativeTransform, SceneComp->GetAttachSocketName());
						}
					}
				}
			}
		}

		// Generate name to node lookup maps for each SCS.  This used to be done just during ExecuteConstruction, but it also optimizes
		// calls to GetArchetype elsewhere during RerunConstructionScripts, so it's advantageous to run it here.
		TArray<const UBlueprintGeneratedClass*> ParentBPClassStack;
		UBlueprintGeneratedClass::GetGeneratedClassesHierarchy(GetClass(), ParentBPClassStack);
		for (const UBlueprintGeneratedClass* BPClass : ParentBPClassStack)
		{
			if (BPClass->SimpleConstructionScript)
			{
				BPClass->SimpleConstructionScript->CreateNameToSCSNodeMap();
			}
		}

		if (!CurrentTransactionAnnotation.IsValid())
		{
			CurrentTransactionAnnotation = FActorTransactionAnnotation::Create(this, false);
		}
		FActorTransactionAnnotation* ActorTransactionAnnotation = CurrentTransactionAnnotation.Get();
		InstanceDataCache = &ActorTransactionAnnotation->ActorTransactionAnnotationData.ComponentInstanceData;

		if (ActorTransactionAnnotation->ActorTransactionAnnotationData.bRootComponentDataCached)
		{
			OldTransform = ActorTransactionAnnotation->ActorTransactionAnnotationData.RootComponentData.Transform;
			OldTransformRotationCache = ActorTransactionAnnotation->ActorTransactionAnnotationData.RootComponentData.TransformRotationCache;
			Parent = ActorTransactionAnnotation->ActorTransactionAnnotationData.RootComponentData.AttachedParentInfo.Actor.Get();
			if (Parent)
			{
				USceneComponent* AttachParent = ActorTransactionAnnotation->ActorTransactionAnnotationData.RootComponentData.AttachedParentInfo.AttachParent.Get();
				AttachParentComponent = (AttachParent ? AttachParent : FindObjectFast<USceneComponent>(Parent, ActorTransactionAnnotation->ActorTransactionAnnotationData.RootComponentData.AttachedParentInfo.AttachParentName));
				SocketName = ActorTransactionAnnotation->ActorTransactionAnnotationData.RootComponentData.AttachedParentInfo.SocketName;
				DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
			}

			for (const FActorRootComponentReconstructionData::FAttachedActorInfo& CachedAttachInfo : ActorTransactionAnnotation->ActorTransactionAnnotationData.RootComponentData.AttachedToInfo)
			{
				AActor* AttachedActor = CachedAttachInfo.Actor.Get();
				if (AttachedActor)
				{
					// Detaching will mark the attached actor's package as dirty, but we don't actually need
					// it to be re-saved after reconstruction since attachment relationships will be restored.
					CheckAndSaveOuterPackageToCleanList(AttachedActor);

					FAttachedActorInfo Info;
					Info.AttachedActor = AttachedActor;
					Info.AttachedToSocket = CachedAttachInfo.SocketName;
					Info.bSetRelativeTransform = true;
					Info.RelativeTransform = CachedAttachInfo.RelativeTransform;
					AttachedActorInfos.Add(Info);

					AttachedActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
				}
			}

			bUseRootComponentProperties = false;
		}

		if (bUseRootComponentProperties)
		{
			// If there are attached objects detach them and store the socket names
			TArray<AActor*> AttachedActors;
			GetAttachedActors(AttachedActors);

			for (AActor* AttachedActor : AttachedActors)
			{
				// Detaching will mark the attached actor's package as dirty, but we don't actually need
				// it to be re-saved after reconstruction since attachment relationships will be restored.
				CheckAndSaveOuterPackageToCleanList(AttachedActor);

				// We don't need to detach child actors, that will be handled by component tear down
				if (!AttachedActor->IsChildActor())
				{
					USceneComponent* EachRoot = AttachedActor->GetRootComponent();
					// If the component we are attached to is about to go away...
					if (EachRoot && EachRoot->GetAttachParent() && EachRoot->GetAttachParent()->IsCreatedByConstructionScript())
					{
						// Save info about actor to reattach
						FAttachedActorInfo Info;
						Info.AttachedActor = AttachedActor;
						Info.AttachedToSocket = EachRoot->GetAttachSocketName();
						Info.bSetRelativeTransform = false;
						if (EachRoot->GetAttachParent() != RootComponent)
						{
							Info.AttachParentName = EachRoot->GetAttachParent()->GetFName();
						}
						AttachedActorInfos.Add(Info);

						// Now detach it
						AttachedActor->Modify();
						EachRoot->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
					}
				}
				else
				{
					check(AttachedActor->ParentComponent->GetOwner() == this);
				}
			}

			if (RootComponent != nullptr)
			{
				// ComponentToWorld must be up-to-date for the detach below, and to get OldTransform
				RootComponent->UpdateComponentToWorld();

				// Do not need to detach if root component is not going away
				if (RootComponent->GetAttachParent() != nullptr && RootComponent->IsCreatedByConstructionScript())
				{
					Parent = RootComponent->GetAttachParent()->GetOwner();
					// Root component should never be attached to another component in the same actor!
					if (Parent == this)
					{
						UE_LOGF(LogActor, Warning, "RerunConstructionScripts: RootComponent (%ls) attached to another component in this Actor (%ls).", *RootComponent->GetPathName(), *Parent->GetPathName());
						Parent = nullptr;
					}
					AttachParentComponent = RootComponent->GetAttachParent();
					SocketName = RootComponent->GetAttachSocketName();
					//detach it to remove any scaling 
					RootComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
				}

				OldTransform = RootComponent->GetComponentTransform();
				OldTransformRotationCache = RootComponent->GetRelativeRotationCache();
			}
		}

		// Return the component which was added by the construction script.
		// It may be the same as the argument, or a parent component if the argument was a native subobject.
		auto GetComponentAddedByConstructionScript = [](UActorComponent* Component) -> UActorComponent*
		{
			while (Component)
			{
				if (Component->IsCreatedByConstructionScript())
				{
					return Component;
				}

				Component = Component->GetTypedOuter<UActorComponent>();
			}

			return nullptr;
		};

		// Build a list of previously attached components which will be matched with their newly instanced counterparts.
		// Components which will be reinstanced may be created by the SCS or the UCS.
		// SCS components can only be matched by name, and outermost parent to resolve duplicated names.
		// UCS components remember a serialized index which is used to identify them in the case that the UCS adds many of the same type.

		TInlineComponentArray<UActorComponent*> PreviouslyAttachedComponents;
		GetComponents(PreviouslyAttachedComponents);

		struct FComponentData
		{
			UActorComponent* OldComponent;
			UActorComponent* OldOuter;
			UObject* OldArchetype;
			FName OldName;
			int32 UCSComponentIndex;
		};

		TArray<FComponentData> ComponentMapping;
		ComponentMapping.Reserve(PreviouslyAttachedComponents.Num());
		int32 IndexOffset = 0;

		for (UActorComponent* Component : PreviouslyAttachedComponents)
		{
			// Look for the outermost component object.
			// Normally components have their parent actor as their outer, but it's possible that a native component may construct a subobject component.
			// In this case we need to "tunnel out" to find the parent component which has been created by the construction script.
			if (UActorComponent* CSAddedComponent = GetComponentAddedByConstructionScript(Component))
			{
				// If we have any instanced components attached to us and we're going to be destroyed we need to explicitly detach them so they don't choose a new
				// parent component and the attachment data we stored in the component instance data cache won't get applied
				if (USceneComponent* SceneComp = Cast<USceneComponent>(Component))
				{
					TInlineComponentArray<USceneComponent*> InstancedChildren;
					Algo::TransformIf(SceneComp->GetAttachChildren(), InstancedChildren, [](USceneComponent* SC) { return SC && SC->CreationMethod == EComponentCreationMethod::Instance; }, [](USceneComponent* SC) { return SC; });
					for (USceneComponent* InstancedChild : InstancedChildren)
					{
						InstancedChild->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
					}
				}


				// Determine if this component is an inner of a component added by the construction script
				const bool bIsInnerComponent = (CSAddedComponent != Component);

				// Try to ensure that children are added to the list after the parents.
				// IndexOffset specifies how many items from the end new items are added.
				const int32 Index = ComponentMapping.Num() - IndexOffset;
				if (bIsInnerComponent)
				{
					int32 OuterIndex = ComponentMapping.IndexOfByPredicate([CSAddedComponent](const FComponentData& CD) { return CD.OldComponent == CSAddedComponent; });
					if (OuterIndex == INDEX_NONE)
					{
						// If we find an item whose parent isn't yet in the list, we put it at the end, and then force all subsequent items to be added before.
						// TODO: improve this, it may fail in certain circumstances, but a full topological ordering is far more complicated a problem.
						IndexOffset++;
					}
				}

				// Add a new item
				FComponentData& ComponentData = ComponentMapping.InsertDefaulted_GetRef(Index);
				ComponentData.OldComponent = Component;
				ComponentData.OldOuter = bIsInnerComponent ? CSAddedComponent : nullptr;
				ComponentData.OldArchetype = Component->GetArchetype();
				ComponentData.OldName = Component->GetFName();
				ComponentData.UCSComponentIndex = Component->GetUCSSerializationIndex();
			}
		}

		// Destroy existing components
		DestroyConstructedComponents();

		// Reset random streams
		ResetPropertiesForConstruction();

		// Cleanup any bindings from this actor to other objects, like bindings to referenced level actors.
		// This is intended to only cleanup bindings by engine code that correspond to user nodes, like
		// Event nodes in blueprint graphs for another actor's events dispatchers. If those nodes still exist
		// the bindings will be rebound on ExecuteConstruction.
#if WITH_EDITOR
		UBlueprintGeneratedClass::UnbindDynamicDelegates(GetClass(), this);
#endif

		// Exchange net roles before running construction scripts
		UWorld *OwningWorld = GetWorld();
		if (OwningWorld && OwningWorld->IsNetMode(NM_Client))
		{
			ExchangeNetRoles(true);
		}

		// Determine if we already have the correct world transform on the RootComponent. If so, we don't want to try to set it again in ExecuteConstruction()
		// or else a re-computation of the relative transform can cause error accumulation on the RelativeLocation/etc which is supposed to derive the ComponentToWorld.
		bool bIsDefaultTransform = false;
		if (RootComponent != nullptr && bUseRootComponentProperties)
		{
			const double TransformTolerance = 0.0;
			if (OldTransform.Equals(RootComponent->GetComponentTransform(), TransformTolerance))
			{
				bIsDefaultTransform = true;
			}
		}

		// Run the construction scripts
		const bool bErrorFree = ExecuteConstruction(OldTransform, &OldTransformRotationCache, InstanceDataCache, bIsDefaultTransform);

		if(Parent)
		{
			USceneComponent* ChildRoot = GetRootComponent();
			if (AttachParentComponent == nullptr)
			{
				AttachParentComponent = Parent->GetRootComponent();
			}
			if (ChildRoot != nullptr && AttachParentComponent != nullptr)
			{
				ChildRoot->AttachToComponent(AttachParentComponent, FAttachmentTransformRules::KeepWorldTransform, SocketName);
			}
		}

		// If we had attached children reattach them now - unless they are already attached
		for(FAttachedActorInfo& Info : AttachedActorInfos)
		{
			// If this actor is no longer attached to anything, reattach
			if (IsValid(Info.AttachedActor) && Info.AttachedActor->GetAttachParentActor() == nullptr)
			{
				USceneComponent* ChildRoot = Info.AttachedActor->GetRootComponent();
				if (ChildRoot && ChildRoot->GetAttachParent() != RootComponent)
				{
					if (Info.AttachParentName != NAME_None)
					{
						TArray<USceneComponent*> ChildComponents;
						RootComponent->GetChildrenComponents(true, ChildComponents);
						for (USceneComponent* Child : ChildComponents)
						{
							if (Child->GetFName() == Info.AttachParentName)
							{
								ChildRoot->AttachToComponent(Child, FAttachmentTransformRules::SnapToTargetIncludingScale, Info.AttachedToSocket);
								break;
							}
						}
						// if we couldn't find component by name, attach it to root and log a warning
						if (!ChildRoot->GetAttachParent())
						{
							UE_LOGF(LogBlueprint, Warning,
								"Couldn't find a component named \'%ls\' when reattaching. Attaching to root component instead.",
								*Info.AttachParentName.ToString()
							);
							ChildRoot->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepWorldTransform, Info.AttachedToSocket);
						}
					}
					else
					{
						ChildRoot->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepWorldTransform, Info.AttachedToSocket);
					}

					if (Info.bSetRelativeTransform)
					{
						ChildRoot->SetRelativeTransform(Info.RelativeTransform);
					}
					ChildRoot->UpdateComponentToWorld();
				}
			}
		}

		// If any of the code above caused a package dirty state to change, we reset it back to a "clean" state now. Note that we have to do this
		// before we restore the undo buffer below - otherwise, the state change will become part of the current transaction, and we don't want that.
		for (UPackage* PackageToMarkAsClean : CleanPackageList)
		{
			check(PackageToMarkAsClean);
			PackageToMarkAsClean->SetDirtyFlag(false);
		}

		CleanPackageList.Empty();

		// Restore the undo buffer
		GUndo = CurrentTransaction;

		// Create the mapping of old->new components and notify the editor of the replacements
		TInlineComponentArray<UActorComponent*> NewComponents;
		GetComponents(NewComponents);

		TMap<UObject*, UObject*> OldToNewComponentMapping;
		OldToNewComponentMapping.Reserve(NewComponents.Num());

		// Build some quick lookup maps for speedy access.
		// The NameToNewComponent map is a multimap because names are not necessarily unique.
		// For example, there may be two components, subobjects of components added by the construction script, which have the same name, because they are unique in their scope.
		TMultiMap<FName, UActorComponent*> NameToNewComponent;
		TMap<UActorComponent*, UObject*> ComponentToArchetypeMap;
		NameToNewComponent.Reserve(NewComponents.Num());
		ComponentToArchetypeMap.Reserve(NewComponents.Num());

		typedef TArray<UActorComponent*, TInlineAllocator<4>> FComponentsForSerializationIndex;
		TMap<int32, FComponentsForSerializationIndex> SerializationIndexToNewUCSComponents;

		for (UActorComponent* NewComponent : NewComponents)
		{
			if (GetComponentAddedByConstructionScript(NewComponent))
			{
				NameToNewComponent.Add(NewComponent->GetFName(), NewComponent);
				ComponentToArchetypeMap.Add(NewComponent, NewComponent->GetArchetype());

				if (NewComponent->CreationMethod == EComponentCreationMethod::UserConstructionScript)
				{
					SerializationIndexToNewUCSComponents.FindOrAdd(NewComponent->GetUCSSerializationIndex()).Add(NewComponent);
				}
			}
		}

		// Now iterate through all previous construction script created components, looking for a match with reinstanced components.
		for (const FComponentData& ComponentData : ComponentMapping)
		{
			UActorComponent* ResolvedNewComponent = nullptr;
			
			if (ComponentData.OldComponent->CreationMethod == EComponentCreationMethod::UserConstructionScript)
			{
				if (ComponentData.UCSComponentIndex >= 0)
				{
					// If created by the UCS, look for a component whose class, archetype, and serialized index matches
					if (FComponentsForSerializationIndex* NewUCSComponentsToConsider = SerializationIndexToNewUCSComponents.Find(ComponentData.UCSComponentIndex))
					{
						for (int32 Index = 0; Index < NewUCSComponentsToConsider->Num(); ++Index)
						{
							UActorComponent* NewComponent = (*NewUCSComponentsToConsider)[Index];
							if (   ComponentData.OldComponent->GetClass() == NewComponent->GetClass() 
							    && ComponentData.OldArchetype == ComponentToArchetypeMap[NewComponent])
							{
								ResolvedNewComponent = NewComponent;
								NewUCSComponentsToConsider->RemoveAtSwap(Index, EAllowShrinking::No);
								break;
							}
						}
					}
				}
			}
			else
			{
				// Component added by the SCS. We can't rely on serialization order as this can change.
				// Instead look for matching names, and, if there's an outer component, look for a match there.
				TArray<UActorComponent*> MatchedComponents;
				NameToNewComponent.MultiFind(ComponentData.OldName, MatchedComponents);
				if (MatchedComponents.Num() > 0)
				{
					UActorComponent* OuterToMatch = ComponentData.OldOuter;
					if (OuterToMatch)
					{
						// The saved outer component is the previous component, hence we transform it to the new one through the OldToNewComponentMapping
						// before comparing with the new outer to match.
						// We can rely on this because the ComponentMapping list is ordered topologically, such that parents appear before children.
						if (UObject** NewOuterToMatch = OldToNewComponentMapping.Find(OuterToMatch))
						{
							OuterToMatch = Cast<UActorComponent>(*NewOuterToMatch);
						}
						else
						{
							OuterToMatch = nullptr;
						}
					}

					// Now look for a match within the set of possible matches
					for (UActorComponent* MatchedComponent : MatchedComponents)
					{
						if (!OuterToMatch || GetComponentAddedByConstructionScript(MatchedComponent) == OuterToMatch)
						{
							ResolvedNewComponent = MatchedComponent;
							break;
						}
					}
				}
			}

			OldToNewComponentMapping.Add(ComponentData.OldComponent, ResolvedNewComponent);
		}

		if (GEditor && (OldToNewComponentMapping.Num() > 0))
		{
			GEditor->NotifyToolsOfObjectReplacement(OldToNewComponentMapping);
		}

		// NotifyToolsOfObjectReplacement works only with UEditorEngine
		// OnComponentsReplaced delegate broadcasts in standalone editor as well
		if (OldToNewComponentMapping.Num() > 0)
		{
			OnComponentsReplaced.Broadcast(OldToNewComponentMapping);
		}

		if (bErrorFree)
		{
			CurrentTransactionAnnotation = nullptr;
		}

		// Remove the name to SCS node maps now that we're done constructing
		for (const UBlueprintGeneratedClass* BPClass : ParentBPClassStack)
		{
			if (BPClass->SimpleConstructionScript)
			{
				BPClass->SimpleConstructionScript->RemoveNameToSCSNodeMap();
			}
		}
	}
}
#endif

namespace
{
	TMap<const AActor*, TMap<const UObject*, int32>, TInlineSetAllocator<4>> UCSBlueprintComponentArchetypeCounts;
}

bool AActor::ExecuteConstruction(const FTransform& Transform, const FRotationConversionCache* TransformRotationCache, const FComponentInstanceDataCache* InstanceDataCache, bool bIsDefaultTransform, ESpawnActorScaleMethod TransformScaleMethod)
{
	check(IsValidChecked(this));
	check(!HasAnyFlags(RF_BeginDestroyed|RF_FinishDestroyed));

#if WITH_EDITOR
	// Guard against reentrancy due to attribute editing at construction time.
	// @see RerunConstructionScripts()
	checkf(!bActorIsBeingConstructed, TEXT("Actor construction is not reentrant"));
#endif
	bActorIsBeingConstructed = true;
	ON_SCOPE_EXIT
	{
		bActorIsBeingConstructed = false;
		UCSBlueprintComponentArchetypeCounts.Remove(this);
	};

	// ensure that any existing native root component gets this new transform
	// we can skip this in the default case as the given transform will be the root component's transform
	if (RootComponent && !bIsDefaultTransform)
	{
		if (TransformRotationCache)
		{
			RootComponent->SetRelativeRotationCache(*TransformRotationCache);
		}
		RootComponent->SetWorldTransform(Transform, /*bSweep=*/false, /*OutSweepHitResult=*/nullptr, ETeleportType::TeleportPhysics);
	}

	// Generate the parent blueprint hierarchy for this actor, so we can run all the construction scripts sequentially
	TArray<const UBlueprintGeneratedClass*> ParentBPClassStack;
	const bool bErrorFree = UBlueprintGeneratedClass::GetGeneratedClassesHierarchy(GetClass(), ParentBPClassStack);

	// If this actor has a blueprint lineage, go ahead and run the construction scripts from least derived to most
	if( (ParentBPClassStack.Num() > 0)  )
	{
		if (bErrorFree)
		{
			// Get all components owned by the given actor prior to SCS execution.
			// Note: GetComponents() internally does a NULL check, so we can assume here that all entries are valid.
			TInlineComponentArray<UActorComponent*> PreSCSComponents;
			GetComponents(PreSCSComponents);

			// Determine the set of native scene components that SCS nodes can attach to.
			TInlineComponentArray<USceneComponent*> NativeSceneComponents;
			for (UActorComponent* ActorComponent : PreSCSComponents)
			{
				if (USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent))
				{
					// Exclude subcomponents of native components, as these could unintentionally be matched by name during SCS execution. Also exclude instance-only components.
					if (SceneComponent->CreationMethod == EComponentCreationMethod::Native && SceneComponent->GetOuter()->IsA<AActor>())
					{
						// If RootComponent is not set, the first unattached native scene component will be used as root. This matches what's done in FixupNativeActorComponents().
												// In cases like BP reparenting between native classes, this is needed to fix up changes in root component type
						if (RootComponent == nullptr && SceneComponent->GetAttachParent() == nullptr)
						{
							// Note: All native scene components should already have been registered at this point, so we don't need to register the component here.
							SetRootComponent(SceneComponent);

							// Update the transform on the newly set root component
							if (ensure(RootComponent) && !bIsDefaultTransform)
							{
								if (TransformRotationCache)
								{
									RootComponent->SetRelativeRotationCache(*TransformRotationCache);
								}
								RootComponent->SetWorldTransform(Transform, /*bSweep=*/false, /*OutSweepHitResult=*/nullptr, ETeleportType::TeleportPhysics);
							}
						}

						NativeSceneComponents.Add(SceneComponent);
					}
				}
			}

			// Prevent user from spawning actors in User Construction Script
			FGuardValue_Bitfield(GetWorld()->bIsRunningConstructionScript, true);
			for (int32 i = ParentBPClassStack.Num() - 1; i >= 0; i--)
			{
				const UBlueprintGeneratedClass* CurrentBPGClass = ParentBPClassStack[i];
				check(CurrentBPGClass);
				USimpleConstructionScript* SCS = CurrentBPGClass->SimpleConstructionScript;
				if (SCS)
				{
					SCS->ExecuteScriptOnActor(this, NativeSceneComponents, Transform, TransformRotationCache, bIsDefaultTransform, TransformScaleMethod);
				}
				// Now that the construction scripts have been run, we can create timelines and hook them up
				UBlueprintGeneratedClass::CreateComponentsForActor(CurrentBPGClass, this);
			}

			// Ensure that we've called RegisterAllComponents(), in case it was deferred and the SCS could not be fully executed.
			if (HasDeferredComponentRegistration() && GetWorld()->bIsWorldInitialized)
			{
				RegisterAllComponents();
			}

			// Once SCS execution has finished, we do a final pass to register any new components that may have been deferred or were otherwise left unregistered after SCS execution.
			TInlineComponentArray<UActorComponent*> PostSCSComponents;
			GetComponents(PostSCSComponents);
			for (UActorComponent* ActorComponent : PostSCSComponents)
			{
				// Limit registration to components that are known to have been created during SCS execution
				if (!ActorComponent->IsRegistered() && ActorComponent->bAutoRegister && IsValidChecked(ActorComponent) && (GetWorld()->bIsWorldInitialized || bHasRegisteredAllComponents)
					&& (ActorComponent->CreationMethod == EComponentCreationMethod::SimpleConstructionScript || !PreSCSComponents.Contains(ActorComponent)))
				{
					USimpleConstructionScript::RegisterInstancedComponent(ActorComponent);
				}
			}

			// If we passed in cached data, we apply it now, so that the UserConstructionScript can use the updated values
			if (InstanceDataCache)
			{
				InstanceDataCache->ApplyToActor(this, ECacheApplyPhase::PostSimpleConstructionScript);
			}

#if WITH_EDITOR
			bool bDoUserConstructionScript;
			GConfig->GetBool(TEXT("Kismet"), TEXT("bTurnOffEditorConstructionScript"), bDoUserConstructionScript, GEngineIni);
			if (!GIsEditor || !bDoUserConstructionScript)
#endif
			{
				// Then run the user script, which is responsible for calling its own super, if desired
				ProcessUserConstructionScript();
			}

			// Since re-run construction scripts will never be run and we want to keep dynamic spawning fast, don't spend time
			// determining the UCS modified properties in game worlds
			if (!GetWorld()->IsGameWorld())
			{
				for (UActorComponent* Component : GetComponents())
				{
					if (Component)
					{
						Component->DetermineUCSModifiedProperties();
					}
				}
			}

			// Bind any delegates on components			
			UBlueprintGeneratedClass::BindDynamicDelegates(GetClass(), this); // We have a BP stack, we must have a UBlueprintGeneratedClass...

			// Apply any cached data procedural components
			// @TODO Don't re-apply to components we already applied to above
			if (InstanceDataCache)
			{
				InstanceDataCache->ApplyToActor(this, ECacheApplyPhase::PostUserConstructionScript);
			}
		}
		else
		{
			// Disaster recovery mode; create a dummy billboard component to retain the actor location
			// until the compile error can be fixed
			if (RootComponent == nullptr)
			{
				UBillboardComponent* BillboardComponent = NewObject<UBillboardComponent>(this);
				BillboardComponent->SetFlags(RF_Transactional);
				BillboardComponent->CreationMethod = EComponentCreationMethod::SimpleConstructionScript;
#if WITH_EDITOR
				BillboardComponent->Sprite = (UTexture2D*)(StaticLoadObject(UTexture2D::StaticClass(), nullptr, TEXT("/Engine/EditorResources/BadBlueprintSprite.BadBlueprintSprite")));
#endif
				BillboardComponent->SetRelativeTransform(Transform);

				SetRootComponent(BillboardComponent);
				FinishAndRegisterComponent(BillboardComponent);
			}

			// Ensure that we've called RegisterAllComponents(), in case it was deferred and the SCS could not be executed (due to error).
			if (HasDeferredComponentRegistration() && GetWorld()->bIsWorldInitialized)
			{
				RegisterAllComponents();
			}
		}
	}
	else
	{
#if WITH_EDITOR
		bool bDoUserConstructionScript;
		GConfig->GetBool(TEXT("Kismet"), TEXT("bTurnOffEditorConstructionScript"), bDoUserConstructionScript, GEngineIni);
		if (!GIsEditor || !bDoUserConstructionScript)
#endif
		{
			// Then run the user script, which is responsible for calling its own super, if desired
			ProcessUserConstructionScript();
		}
		UBlueprintGeneratedClass::BindDynamicDelegates(GetClass(), this);
	}

	GetWorld()->UpdateCullDistanceVolumes(this);

	// Now run virtual notification
	OnConstruction(Transform);

	return bErrorFree;
}

void AActor::ProcessUserConstructionScript()
{
#if WITH_EDITOR
	TOptional<UE::Cook::FCookRandScope> CookRandScope;
	if (IsRunningCookCommandlet())
	{
		// Notify blueprint functions called by the UserConstructScript that support CookRand to return deterministic
		// results based on our seed. Overwrite an existing CookRand if it exists so that we always get the same
		// guids/rands even if called from other UserConstruction scripts or CookRand environments.
		int32 Seed = UE::Cook::FCookRand::ConstructSeed(GetPathName());
		CookRandScope.Emplace(Seed, UE::Cook::ECookRandScope::Overwrite);
	}
#endif
	// Set a flag that this actor is currently running UserConstructionScript.
	bRunningUserConstructionScript = true;
	UserConstructionScript();
	bRunningUserConstructionScript = false;

	// Validate component mobility after UCS execution
	for (UActorComponent* Component : GetComponents())
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			// A parent component can't be more mobile than its children, so we check for that here and adjust as needed.
			if (SceneComponent != RootComponent && SceneComponent->GetAttachParent() != nullptr && SceneComponent->GetAttachParent()->Mobility > SceneComponent->Mobility)
			{
				if (SceneComponent->IsA<UStaticMeshComponent>())
				{
					// SMCs can't be stationary, so always set them (and any children) to be movable
					SceneComponent->SetMobility(EComponentMobility::Movable);
				}
				else
				{
					// Set the new component (and any children) to be at least as mobile as its parent
					SceneComponent->SetMobility(SceneComponent->GetAttachParent()->Mobility);
				}
			}
		}
	}
}

void AActor::FinishAndRegisterComponent(UActorComponent* Component)
{
	if (GetWorld()->bIsWorldInitialized)
	{
		Component->RegisterComponent();
	}
	BlueprintCreatedComponents.Add(Component);
}

#if !UE_BUILD_SHIPPING
static TAutoConsoleVariable<int32> CVarLogBlueprintComponentInstanceCalls(
	TEXT("LogBlueprintComponentInstanceCalls"),
	0,
	TEXT("Log Blueprint Component instance calls; debugging."));
#endif

static FName FindFirstFreeName(UObject* Outer, FName BaseName)
{
	int32 Lower = 0;
	FName Ret = FName(BaseName, Lower);

	// Binary search if we appear to have used this name a lot, else
	// just linear search for the first free index:
	if (FindObjectFast<UObject>(Outer, FName(BaseName, 100)))
		{
		// could be replaced by Algo::LowerBound if TRange or some other
		// integer range type could be made compatible with the algo's
		// implementation:
		int32 Upper = INT_MAX;
		while (true)
		{
			int32 Next = (Upper - Lower) / 2 + Lower;
			if (FindObjectFast<UObject>(Outer, FName(BaseName, Next)))
			{
				Lower = Next + 1;
			}
			else
			{
				Upper = Next;
			}

			if (Upper == Lower)
			{
				Ret = FName(BaseName, Lower);
				break;
			}
		}
	}
	else
	{
		while (FindObjectFast<UObject>(Outer, Ret))
		{
			Ret = FName(BaseName, ++Lower);
		}
	}

	return Ret;
}

UActorComponent* AActor::CreateComponentFromTemplate(UActorComponent* Template, FName InName)
{
	SCOPE_CYCLE_COUNTER(STAT_InstanceActorComponent);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(InstanceActorComponent);

	UActorComponent* NewActorComp = nullptr;
	if (Template != nullptr)
	{
#if !UE_BUILD_SHIPPING
		const double StartTime = FPlatformTime::Seconds();
#endif
		if (InName == NAME_None)
		{
			// Search for a unique name based on our template. Our template is going to be our archetype
			// thanks to logic in UBPGC::FindArchetype:
			InName = FindFirstFreeName(this, Template->GetFName());
		}
		else
		{
			// Resolve any name conflicts.
			CheckComponentInstanceName(InName);
		}

		// Note we aren't copying the the RF_ArchetypeObject flag. Also note the result is non-transactional by default.
		FObjectDuplicationParameters DupeActorParameters(Template, this);
		DupeActorParameters.DestName = InName;
		DupeActorParameters.FlagMask = RF_AllFlags & ~(RF_ArchetypeObject | RF_Transactional | RF_WasLoaded | RF_Public | RF_InheritableComponentTemplate);
		DupeActorParameters.PortFlags = PPF_DuplicateVerbatim; // Skip resetting text IDs
		NewActorComp = (UActorComponent*)StaticDuplicateObjectEx(DupeActorParameters);

		// Handle post-creation tasks.
		PostCreateBlueprintComponent(NewActorComp);

#if !UE_BUILD_SHIPPING
		if (CVarLogBlueprintComponentInstanceCalls.GetValueOnGameThread())
		{
			UE_LOGF(LogBlueprint, Log, "%ls: CreateComponentFromTemplate() - %ls \'%ls\' completed in %.02g ms", *GetName(), *Template->GetClass()->GetName(), *InName.ToString(), (FPlatformTime::Seconds() - StartTime) * 1000.0);
		}
#endif
	}
	return NewActorComp;
}

UActorComponent* AActor::CreateComponentFromTemplateData(const FBlueprintCookedComponentInstancingData* TemplateData, FName InName)
{
	SCOPE_CYCLE_COUNTER(STAT_InstanceActorComponent);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(InstanceActorComponent);

	// Component instance data loader implementation.
	class FBlueprintComponentInstanceDataLoader : public FObjectReader
	{
	public:
		FBlueprintComponentInstanceDataLoader(const TArray<uint8>& InSrcBytes, const FCustomPropertyListNode* InPropertyList)
			:FObjectReader(const_cast<TArray<uint8>&>(InSrcBytes))
		{
			ArCustomPropertyList = InPropertyList;
			ArUseCustomPropertyList = true;
			this->SetWantBinaryPropertySerialization(true);

			// Set this flag to emulate things that would happen in the SDO case when this flag is set (e.g. - not setting 'bHasBeenCreated').
			ArPortFlags |= PPF_Duplicate;

			// Skip resetting text IDs
			ArPortFlags |= PPF_DuplicateVerbatim;
		}
	};

	UActorComponent* NewActorComp = nullptr;
	if (TemplateData != nullptr && TemplateData->ComponentTemplateClass != nullptr)	// some components (e.g. UTextRenderComponent) are not loaded on a server (or client). Handle that gracefully, but we ideally shouldn't even get here (see UEBP-175).
	{
#if !UE_BUILD_SHIPPING
		const double StartTime = FPlatformTime::Seconds();
#endif
		// Resolve any name conflicts.
		if (InName == NAME_None)
		{
			// Search for a unique name based on our template. Our template is going to be our archetype
			// thanks to logic in UBPGC::FindArchetype:
			InName = FindFirstFreeName(this, TemplateData->ComponentTemplateName);
		}
		else
		{
			// Resolve any name conflicts.
			CheckComponentInstanceName(InName);
		}

		// Note we aren't copying the the RF_ArchetypeObject flag. Also note the result is non-transactional by default.
		NewActorComp = NewObject<UActorComponent>(
			this,
			TemplateData->ComponentTemplateClass,
			InName,
			EObjectFlags(TemplateData->ComponentTemplateFlags) & ~(RF_ArchetypeObject | RF_Transactional | RF_WasLoaded | RF_Public | RF_InheritableComponentTemplate)
		);

		// Set these flags to match what SDO would otherwise do before serialization to enable post-duplication logic on the destination object.
		NewActorComp->SetFlags(RF_NeedPostLoad | RF_NeedPostLoadSubobjects);

		// Load cached data into the new instance.
		FBlueprintComponentInstanceDataLoader ComponentInstanceDataLoader(TemplateData->GetCachedPropertyData(), TemplateData->GetCachedPropertyList());
		NewActorComp->Serialize(ComponentInstanceDataLoader);

		// Instance any subobject values initialized from template data.
		if (NewActorComp->GetClass()->HasAnyClassFlags(CLASS_HasInstancedReference))
		{
			NewActorComp->InstanceSubobjectTemplates();
		}

		// Handle tasks that would normally occur post-duplication w/ SDO.
		NewActorComp->PostDuplicate(EDuplicateMode::Normal);
		{
			TGuardValue<bool> GuardIsRoutingPostLoad(FUObjectThreadContext::Get().IsRoutingPostLoad, true);
			NewActorComp->ConditionalPostLoad();
		}

		// Handle post-creation tasks.
		PostCreateBlueprintComponent(NewActorComp);

#if !UE_BUILD_SHIPPING
		if (CVarLogBlueprintComponentInstanceCalls.GetValueOnGameThread())
		{
			UE_LOGF(LogBlueprint, Log, "%ls: CreateComponentFromTemplateData() - %ls \'%ls\' completed in %.02g ms", *GetName(), *TemplateData->ComponentTemplateClass->GetName(), *InName.ToString(), (FPlatformTime::Seconds() - StartTime) * 1000.0);
		}
#endif
	}
	return NewActorComp;
}

UActorComponent* AActor::AddComponent(FName TemplateName, bool bManualAttachment, const FTransform& RelativeTransform, const UObject* ComponentTemplateContext, bool bDeferredFinish)
{
	if (const UWorld* World = GetWorld())
	{
		if (World->bIsTearingDown)
		{
			UE_LOGF(LogActor, Warning, "AddComponent failed for actor: [%ls] with param TemplateName: [%ls] because we are in the process of tearing down the world"
				, *GetName()
				, *TemplateName.ToString());
			return nullptr;
		}
	}
	else
	{
		UE_LOGF(LogActor, Warning, "AddComponent failed for actor: [%ls] with param TemplateName: [%ls] because world == nullptr"
			, *GetName()
			, *TemplateName.ToString());
		return nullptr;
	}

	UActorComponent* Template = nullptr;
	FBlueprintCookedComponentInstancingData* TemplateData = nullptr;
	for (UClass* TemplateOwnerClass = (ComponentTemplateContext != nullptr) ? ComponentTemplateContext->GetClass() : GetClass()
		; TemplateOwnerClass && !Template && !TemplateData
		; TemplateOwnerClass = TemplateOwnerClass->GetSuperClass())
	{
		if (UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(TemplateOwnerClass))
		{
			// Use cooked instancing data if available (fast path).
			if (BPGC->UseFastPathComponentInstancing())
			{
				TemplateData = BPGC->CookedComponentInstancingData.Find(TemplateName);
			}
			
			if (!TemplateData || !TemplateData->bHasValidCookedData
				|| !ensureMsgf(TemplateData->ComponentTemplateClass != nullptr, TEXT("AddComponent fast path (%s.%s): Cooked data is valid, but runtime support data is not initialized. Using the slow path instead."), *BPGC->GetName(), *TemplateName.ToString()))
			{
				Template = BPGC->FindComponentTemplateByName(TemplateName);
			}
		}
	}

	UActorComponent* NewActorComp = TemplateData ? CreateComponentFromTemplateData(TemplateData) : CreateComponentFromTemplate(Template);

	if (!bDeferredFinish)
	{
		FinishAddComponent(NewActorComp, bManualAttachment, RelativeTransform);
	}

	return NewActorComp;
}

UActorComponent* AActor::AddComponentByClass(TSubclassOf<UActorComponent> Class, bool bManualAttachment, const FTransform& RelativeTransform, bool bDeferredFinish)
{
	if (Class == nullptr)
	{
		return nullptr;
	}

	if (const UWorld* World = GetWorld())
	{
		if (World->bIsTearingDown)
		{
			UE_LOGF(LogActor, Warning, "AddComponentByClass failed for actor: [%ls] with param Class: [%ls] because we are in the process of tearing down the world"
				, *GetName()
				, *GetNameSafe(Class));
			return nullptr;
		}
	}
	else
	{
		UE_LOGF(LogActor, Warning, "AddComponentByClass failed for actor: [%ls] with param Class: [%ls] because world == nullptr"
			, *GetName()
			, *GetNameSafe(Class));
		return nullptr;
	}

	UActorComponent* NewActorComp = NewObject<UActorComponent>(this, *Class);
	PostCreateBlueprintComponent(NewActorComp);

	if (!bDeferredFinish)
	{
		FinishAddComponent(NewActorComp, bManualAttachment, RelativeTransform);
	}

	return NewActorComp;
}

void AActor::FinishAddComponent(UActorComponent* NewActorComp, bool bManualAttachment, const FTransform& RelativeTransform)
{
	if(NewActorComp != nullptr)
	{
		bool bIsSceneComponent = false;

		// Call function to notify component it has been created
		NewActorComp->OnComponentCreated();
		
		// The user has the option of doing attachment manually where they have complete control or via the automatic rule
		// that the first component added becomes the root component, with subsequent components attached to the root.
		USceneComponent* NewSceneComp = Cast<USceneComponent>(NewActorComp);
		if(NewSceneComp != nullptr)
		{
			if (!bManualAttachment)
			{
				if (RootComponent == nullptr)
				{
					RootComponent = NewSceneComp;
				}
				else
				{
					NewSceneComp->SetupAttachment(RootComponent);
				}
			}

			NewSceneComp->SetRelativeTransform(RelativeTransform);

			bIsSceneComponent = true;
		}

		// Register component, which will create physics/rendering state, now component is in correct position
		if (NewActorComp->bAutoRegister)
		{
			NewActorComp->RegisterComponent();
		}

		UWorld* World = GetWorld();
		if (!bRunningUserConstructionScript && World && bIsSceneComponent)
		{
			UPrimitiveComponent* NewPrimitiveComponent = Cast<UPrimitiveComponent>(NewActorComp);
			if (NewPrimitiveComponent && ACullDistanceVolume::CanBeAffectedByVolumes(NewPrimitiveComponent))
			{
				World->UpdateCullDistanceVolumes(this, NewPrimitiveComponent);
			}
		}
	}
}

void AActor::CheckComponentInstanceName(const FName InName)
{
	// If there is a Component with this name already (almost certainly because it is an Instance component), we need to rename it out of the way
	if (!InName.IsNone())
	{
		UObject* ConflictingObject = FindObjectFast<UObject>(this, InName);
		if (ConflictingObject && ConflictingObject->IsA<UActorComponent>() && CastChecked<UActorComponent>(ConflictingObject)->CreationMethod == EComponentCreationMethod::Instance)
		{
			// Try and pick a good name
			FString ConflictingObjectName = ConflictingObject->GetName();
			int32 CharIndex = ConflictingObjectName.Len() - 1;
			while (CharIndex >= 0 && FChar::IsDigit(ConflictingObjectName[CharIndex]))
			{
				--CharIndex;
			}
			// Name is only composed of digits not a name conflict resolution
			if (CharIndex < 0)
			{
				return;
			}

			int32 Counter = 0;
			if (CharIndex < ConflictingObjectName.Len() - 1)
			{
				Counter = FCString::Atoi(*ConflictingObjectName.RightChop(CharIndex + 1));
				ConflictingObjectName.LeftInline(CharIndex + 1, EAllowShrinking::No);
			}
			FString NewObjectName;
			do
			{
				NewObjectName = ConflictingObjectName + FString::FromInt(++Counter);

			} while (FindObjectFast<UObject>(this, *NewObjectName) != nullptr);

			ConflictingObject->Rename(*NewObjectName, this);
		}
	}
}

struct FSetUCSSerializationIndex
{
	friend class AActor;

private:
	FORCEINLINE static void Set(UActorComponent* Component, int32 SerializationIndex)
	{
		Component->UCSSerializationIndex = SerializationIndex;
	}
};

void AActor::PostCreateBlueprintComponent(UActorComponent* NewActorComp)
{
	if (NewActorComp)
	{
		NewActorComp->CreationMethod = EComponentCreationMethod::UserConstructionScript;

		// Need to do this so component gets saved - Components array is not serialized
		BlueprintCreatedComponents.Add(NewActorComp);

		if (bActorIsBeingConstructed)
		{
			TMap<const UObject*, int32>& ComponentArchetypeCounts = UCSBlueprintComponentArchetypeCounts.FindOrAdd(this);
			int32& Count = ComponentArchetypeCounts.FindOrAdd(NewActorComp->GetArchetype());
			FSetUCSSerializationIndex::Set(NewActorComp, Count);
			++Count;

			NewActorComp->SetNetAddressable();
		}

		// The component may not have been added to ReplicatedComponents if it was duplicated from
		// a template, since ReplicatedComponents is normally only updated before the duplicated properties
		// are copied over - in this case bReplicates would not have been set yet, but it will be now.
		if (NewActorComp->GetIsReplicated())
		{
			ReplicatedComponents.AddUnique(NewActorComp);

			AddComponentForReplication(NewActorComp);
		}
	}
}

