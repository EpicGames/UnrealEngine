// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorDeferredScriptManager.h"
#include "GameFramework/Actor.h"
#include "StaticMeshCompiler.h"

#if WITH_EDITOR

#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"
#include "Engine/Level.h"
#include "Templates/GuardValueAccessors.h"
#include "Engine/StaticMesh.h"
#include "Logging/LogMacros.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Engine/World.h"

#define LOCTEXT_NAMESPACE "ActorDeferredScriptManager"

DEFINE_LOG_CATEGORY_STATIC(LogActorDeferredScriptManager, Log, All);

FActorDeferredScriptManager& FActorDeferredScriptManager::Get()
{
	static FActorDeferredScriptManager Singleton;
	return Singleton;
}

void FActorDeferredScriptManager::Shutdown()
{
	if (OnAssetChangeDelegateHandle.IsValid())
	{
		FAssetCompilingManager::Get().OnAssetPostCompileEvent().Remove(OnAssetChangeDelegateHandle);
		OnAssetChangeDelegateHandle.Reset();
	}

	if (OnWorldCleanupDelegateHandle.IsValid())
	{
		FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupDelegateHandle);
		OnWorldCleanupDelegateHandle.Reset();
	}
}

FActorDeferredScriptManager::FActorDeferredScriptManager()
	: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	OnWorldCleanupDelegateHandle = FWorldDelegates::OnWorldCleanup.AddRaw(this, &FActorDeferredScriptManager::OnWorldCleanup);
}

FName FActorDeferredScriptManager::GetStaticAssetTypeName()
{
	return TEXT("UE-ActorConstructionScripts");
}

void FActorDeferredScriptManager::OnWorldCleanup(class UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	int32 RemovedCount = 
		PendingConstructionScriptActors.RemoveAll(
			[InWorld](TWeakObjectPtr<AActor>& InActor)
			{
				AActor* Actor = InActor.Get();
				if (Actor == nullptr)
				{
					UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Removed an actor from deferred construction script because its not valid anymore");
					return true;
				}
				else if (Actor->GetWorld() == InWorld)
				{
					UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Removed %ls from deferred construction script because its world is being cleaned up", *InActor->GetPathName());
					return true;
				}

				return false;
			}
		);

	if (RemovedCount > 0)
	{
		// Whenever an actor is removed, reset the count so we go over all actors on next ProcessAsyncTasks
		NumLeftToProcess = PendingConstructionScriptActors.Num();

		UpdateCompilationNotification();
	}
}

void FActorDeferredScriptManager::AddActor(AActor* InActor)
{
	UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Adding actor %ls for deferred construction script", *InActor->GetPathName());

	PendingConstructionScriptActors.Add(InActor);

	// Whenever an actor is added, reset the count so we go over all actors on next ProcessAsyncTasks
	NumLeftToProcess = PendingConstructionScriptActors.Num();

	UpdateCompilationNotification();
}

FName FActorDeferredScriptManager::GetAssetTypeName() const
{
	return GetStaticAssetTypeName();
}

FTextFormat FActorDeferredScriptManager::GetAssetNameFormat() const
{
	return LOCTEXT("ScriptNameFormat", "{0}|plural(one=Construction Script,other=Construction Scripts)");
}

TArrayView<FName> FActorDeferredScriptManager::GetDependentTypeNames() const
{
	static FName DependentTypeNames[] =
	{
		// Construction scripts needs to wait until static meshes are done prior to running
		FStaticMeshCompilingManager::GetStaticAssetTypeName()
	};
	return TArrayView<FName>(DependentTypeNames);
}

int32 FActorDeferredScriptManager::GetNumRemainingAssets() const
{
	return PendingConstructionScriptActors.Num();
}

void FActorDeferredScriptManager::OnAssetPostCompile(const TArray<FAssetCompileData>& CompiledAssets)
{
	if (!PendingConstructionScriptActors.IsEmpty())
	{
		for (const FAssetCompileData& CompileData : CompiledAssets)
		{
			if (UObject* Object = CompileData.Asset.Get())
			{
				if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
				{
					// Whenever a static mesh finish compiling, process everything again
					NumLeftToProcess = PendingConstructionScriptActors.Num();
					break;
				}
			}
		}
	}
}

void FActorDeferredScriptManager::EnsureEventRegistered()
{
	if (!OnAssetChangeDelegateHandle.IsValid())
	{
		OnAssetChangeDelegateHandle = FAssetCompilingManager::Get().OnAssetPostCompileEvent().AddRaw(this, &FActorDeferredScriptManager::OnAssetPostCompile);
	}
}

void FActorDeferredScriptManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	if (PendingConstructionScriptActors.IsEmpty())
	{
		return;
	}

	EnsureEventRegistered();

	const bool bIsStaticMeshesCompiling = FStaticMeshCompilingManager::Get().GetNumRemainingMeshes() > 0;

	// If this happens, it means OnAssetPostCompile didn't do it's job... better safe than sorry
	if (NumLeftToProcess == 0 && !bIsStaticMeshesCompiling)
	{
		NumLeftToProcess = PendingConstructionScriptActors.Num();
	}

	if (NumLeftToProcess > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FActorDeferredScriptManager::ProcessAsyncTasks);

		// Cache outers result to reduce cost
		TMap<ULevel*, bool> LevelToCompilationLeft;
		auto DoesLevelHaveCompilationLeft =
			[&LevelToCompilationLeft](ULevel* Level)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(DoesLevelHaveCompilationLeft);

				if (bool* bLevelHasCompilationLeft = LevelToCompilationLeft.Find(Level))
				{
					return *bLevelHasCompilationLeft;
				}

				bool bLevelHasCompilationLeft = Level->HasStaticMeshCompilationPending();
				LevelToCompilationLeft.Add(Level, bLevelHasCompilationLeft);
				return bLevelHasCompilationLeft;
			};

		double TickStartTime = FPlatformTime::Seconds();
		const double MaxSecondsPerFrame = 0.016;

		// Since this deferred run of construction script was supposed to be done during level load
		// temporarily set the global flag to prevent dirtying the level package.
		TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);

		int32 CurrentIndex = NextIndexToProcess;
		UPackage* EnteredPackage = nullptr;
		FAssetCompilingManager::FPackageScopeEvent& OnPackageScopeEvent = FAssetCompilingManager::Get().OnPackageScopeEvent();

		UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Starting to process at index %d", CurrentIndex);

		NextIndexToProcess = 0;
		NumLeftToProcess = FMath::Min(NumLeftToProcess, PendingConstructionScriptActors.Num());

		// Always process all items if time permits
		for (; NumLeftToProcess > 0; --NumLeftToProcess, ++CurrentIndex)
		{
			// Act as a circular buffer so we process all items if we start at an offset.
			const int32 CircularIndex = CurrentIndex % PendingConstructionScriptActors.Num();

			const bool bHasTimeLeft = bLimitExecutionTime ? ((FPlatformTime::Seconds() - TickStartTime) < MaxSecondsPerFrame) : true;
			if (!bHasTimeLeft)
			{
				// We're not finished, save where we're at for next time
				UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "No time left, will continue at index %d", CircularIndex);
				NextIndexToProcess = CircularIndex;
				break;
			}

			if (AActor* Actor = PendingConstructionScriptActors[CircularIndex].Get(); Actor && Actor->GetWorld())
			{
				UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Processing index %d (%ls)", CircularIndex, *Actor->GetPathName());

				// Don't do this costly step if we know there is no more meshes compiling
				if (bIsStaticMeshesCompiling)
				{
					if (ULevel* Level = Actor->GetLevel())
					{
						if (DoesLevelHaveCompilationLeft(Level))
						{
							// Do not rerun construction scripts until all the meshes in the level have finished compiling
							UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Skipping index %d (%ls) because it's level outer still has static meshes compilation pending", CircularIndex, *Actor->GetPathName());
							continue;

						}
					}
				}

				UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Rerunning construction script for index %d (%ls)", CircularIndex, *Actor->GetPathName());

				UPackage* Package = Actor->GetPackage();
				if (Package != EnteredPackage)
				{
					if (EnteredPackage)
					{
						OnPackageScopeEvent.Broadcast(EnteredPackage, false /* bEnter */);
					}
					EnteredPackage = Package;
					if (EnteredPackage)
					{
						OnPackageScopeEvent.Broadcast(EnteredPackage, true /* bEnter */);
					}
				}

				// Temporarily do not consider actor as initialized if they were when running deferred construction scripts
				FGuardValue_Bitfield(Actor->bActorInitialized, false);
				Actor->RerunConstructionScripts();
			}
			else
			{
				UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Removing index %d because its actor is not valid anymore", CircularIndex);
			}

			// Reduce reallocations by only shrinking when removing last item
			const EAllowShrinking AllowShrinking = (PendingConstructionScriptActors.Num() == 1) ? EAllowShrinking::Yes : EAllowShrinking::No;
			UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Removing index %d", CircularIndex);
			PendingConstructionScriptActors.RemoveAtSwap(CircularIndex, 1, AllowShrinking);
			CurrentIndex--;

			if (PendingConstructionScriptActors.IsEmpty())
			{
				UE_LOGF(LogActorDeferredScriptManager, VeryVerbose, "Finished processing construction scripts");
			}
		}
		if (EnteredPackage)
		{
			OnPackageScopeEvent.Broadcast(EnteredPackage, false /* bEnter */);
		}
	}

	UpdateCompilationNotification();
}

TRACE_DECLARE_INT_COUNTER(QueuedConstructionScripts, TEXT("AsyncCompilation/QueuedConstructionScripts"));
void FActorDeferredScriptManager::UpdateCompilationNotification()
{
	TRACE_COUNTER_SET(QueuedConstructionScripts, GetNumRemainingAssets());
	Notification->Update(GetNumRemainingAssets());
}

#undef LOCTEXT_NAMESPACE

#endif // #if WITH_EDITOR
