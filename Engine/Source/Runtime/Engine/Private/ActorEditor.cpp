// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Level.h"
#include "RenderingThread.h"
#include "Math/RotationMatrix.h"
#include "SceneInterface.h"
#include "UObject/Linker.h"
#include "Engine/Blueprint.h"
#include "Components/PrimitiveComponent.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "EditorSupportDelegates.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "LevelUtils.h"
#include "Misc/MapErrors.h"
#include "ActorEditorUtils.h"

#if WITH_EDITOR

#include "Editor.h"
#include "ActorTransactionAnnotation.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "WorldPartition/DataLayer/IDataLayerEditorModule.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "ActorFolder.h"
#include "WorldPersistentFolders.h"
#include "Modules/ModuleManager.h"
#include "Engine/SimpleConstructionScript.h"
#include "StaticMeshResources.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "UObject/SoftObjectPtr.h"
#include "Algo/Transform.h"
#include "Algo/RemoveIf.h"
#include "Misc/DataValidation.h"
#include "DeletedObjectPlaceholder.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#define LOCTEXT_NAMESPACE "ErrorChecking"

namespace ActorEditorSettings
{

static bool bIncludeSCSModifiedPropertiesInDiff = true;
static FAutoConsoleVariableRef CVarIncludeSCSModifiedPropertiesInDiff(TEXT("Actor.IncludeSCSModifiedPropertiesInDiff"), bIncludeSCSModifiedPropertiesInDiff, TEXT("True to include SCS modified properties in any transaction diffs, or False to skip them"));

}

void AActor::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	FObjectProperty* ObjProp = CastField<FObjectProperty>(PropertyThatWillChange);
	UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(GetClass());
	if ( BPGC != nullptr && ObjProp != nullptr )
	{
		BPGC->UnbindDynamicDelegatesForProperty(this, ObjProp);
	}

	// During SIE, allow components to be unregistered here, and then reregistered and reconstructed in PostEditChangeProperty.
	if ((GEditor && GEditor->bIsSimulatingInEditor) || ReregisterComponentsWhenModified())
	{
		UnregisterAllComponents();
	}

	PreEditChangeDataLayers.Reset();
	if (PropertyThatWillChange != nullptr && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, DataLayerAssets))
	{
		PreEditChangeDataLayers = DataLayerAssets;
	}
}

bool AActor::CanEditChangeComponent(const UActorComponent* Component, const FProperty* InProperty) const
{
	// Actors in LevelInstances can't be edited unless they are in any kind of editing level instance (edit or property override)
	if (IsInLevelInstance() && !IsInAnyEditLevelInstance())
	{
		return false;
	}

	return true;
}

bool AActor::CanEditChange(const FProperty* PropertyThatWillChange) const
{
	if ((PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, Layers)) ||
		(PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, ActorGuid)))
	{
		return false;
	}

	const bool bIsSpatiallyLoadedProperty = PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, bIsSpatiallyLoaded);
	const bool bIsRuntimeGridProperty = PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, RuntimeGrid);
	const bool bIsDataLayersProperty = PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, DataLayerAssets);
	const bool bIsHLODLayerProperty = PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, HLODLayer);

	if (bIsSpatiallyLoadedProperty || bIsRuntimeGridProperty || bIsDataLayersProperty || bIsHLODLayerProperty)
	{
		if (!IsTemplate())
		{
			UWorld* OwningWorld = GetWorld();
			UWorld* OuterWorld = GetTypedOuter<UWorld>();
			UWorldPartition* OwningWorldPartition = OwningWorld ? OwningWorld->GetWorldPartition() : nullptr;
			UWorldPartition* OuterWorldPartition = OuterWorld ? OuterWorld->GetWorldPartition() : nullptr;			

			if (!bIsSpatiallyLoadedProperty && (!OwningWorldPartition || !OuterWorldPartition))
			{
				return false;
			}

			if (GetAttachParentActor())
			{
				if (bIsSpatiallyLoadedProperty || bIsRuntimeGridProperty)
				{
					return false;
				}
			}
		}
	}

	if (bIsSpatiallyLoadedProperty && !CanChangeIsSpatiallyLoadedFlag())
	{
		return false;
	}

	if (bIsDataLayersProperty && (!SupportsDataLayerType(UDataLayerInstanceWithAsset::StaticClass()) || !IsUserManaged() || GetAttachParentActor()))
	{
		return false;
	}

	// Actors in LevelInstances can't be edited unless they are in any kind of editing level instance (edit or property override)
	if (IsInLevelInstance() && !IsInAnyEditLevelInstance())
	{
		return false;
	}

	return Super::CanEditChange(PropertyThatWillChange);
}

void AActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName Name_ActorLabel = GET_MEMBER_NAME_CHECKED(AActor, ActorLabel);
	static const FName Name_RelativeLocation = USceneComponent::GetRelativeLocationPropertyName();
	static const FName Name_RelativeRotation = USceneComponent::GetRelativeRotationPropertyName();
	static const FName Name_RelativeScale3D = USceneComponent::GetRelativeScale3DPropertyName();
	static const FName Name_MigratingAsset = GET_MEMBER_NAME_CHECKED(AActor, bMigratingAsset);	

	FProperty* MemberPropertyThatChanged = PropertyChangedEvent.MemberProperty;
	const FName MemberPropertyName = MemberPropertyThatChanged != NULL ? MemberPropertyThatChanged->GetFName() : NAME_None;

	if (IsPropertyChangedAffectingDataLayers(PropertyChangedEvent))
	{
		FixupDataLayers(/*bRevertChangesOnLockedDataLayer*/true);
	}

	const bool bTransformationChanged = (MemberPropertyName == Name_RelativeLocation || MemberPropertyName == Name_RelativeRotation || MemberPropertyName == Name_RelativeScale3D);

	// During SIE, allow components to reregistered and reconstructed in PostEditChangeProperty.
	// This is essential as construction is deferred during spawning / duplication when in SIE.
	const bool bReregisterComponents = (MemberPropertyName != Name_ActorLabel) && (((GEditor && GEditor->bIsSimulatingInEditor && GetWorld() != nullptr) || ReregisterComponentsWhenModified()));

	if (bReregisterComponents)
	{
		// In the Undo case we have an annotation storing information about constructed components and we do not want
		// to improperly apply out of date changes so we need to skip registration of all blueprint created components
		// and defer instance components attached to them until after rerun
		if (CurrentTransactionAnnotation.IsValid())
		{
			UnregisterAllComponents();

			TInlineComponentArray<UActorComponent*> Components;
			GetComponents(Components);

			Components.Sort([](UActorComponent& A, UActorComponent& B)
			{
				if (&B == B.GetOwner()->GetRootComponent())
				{
					return false;
				}
				if (USceneComponent* ASC = Cast<USceneComponent>(&A))
				{
					if (ASC->GetAttachParent() == &B)
					{
						return false;
					}
				}
				return true;
			});

			bool bRequiresReregister = false;
			for (UActorComponent* Component : Components)
			{
				if (Component->CreationMethod == EComponentCreationMethod::Native)
				{
					Component->RegisterComponent();
				}
				else if (Component->CreationMethod == EComponentCreationMethod::Instance)
				{
					USceneComponent* SC = Cast<USceneComponent>(Component);
					if (SC == nullptr || SC == RootComponent || (SC->GetAttachParent() && SC->GetAttachParent()->IsRegistered()))
					{
						Component->RegisterComponent();
					}
					else
					{
						bRequiresReregister = true;
					}
				}
				else
				{
					bRequiresReregister = true;
				}
			}

			RerunConstructionScripts();

			if (bRequiresReregister)
			{
				ReregisterAllComponents();
			}
			else
			{
				bHasRegisteredAllComponents = true;
				PostRegisterAllComponents();
			}
		}
		else
		{
			UnregisterAllComponents();
			RerunConstructionScripts();
			ReregisterAllComponents();
		}
	}

	// Let other systems know that an actor was moved
	if (bTransformationChanged)
	{
		GEngine->BroadcastOnActorMoved( this );
	}

	if (MemberPropertyName == Name_MigratingAsset)
	{
		if (bMigratingAsset)
		{
			SetFlags(RF_MigratingAsset);
		}
		else
		{
			ClearFlags(RF_MigratingAsset);
		}
	}

	FEditorSupportDelegates::UpdateUI.Broadcast();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void AActor::PostEditMove(bool bFinished)
{
	if ( ReregisterComponentsWhenModified() && !FLevelUtils::IsMovingLevel())
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(GetClass()->ClassGeneratedBy);
		if (bFinished || bRunConstructionScriptOnDrag || (Blueprint && Blueprint->bRunConstructionScriptOnDrag))
		{
			// Set up a bulk reregister context to optimize RerunConstructionScripts.  We can't move this inside RerunConstructionScripts
			// for the general case, because in other code paths, we use a context to cover additional code outside this function (a second
			// pass that overrides instance data, generating additional render commands on the same set of components).  To move it, we
			// would need to support nesting of these contexts.
			TInlineComponentArray<UActorComponent*> ActorComponents;
			GetComponents(ActorComponents);
			FStaticMeshComponentBulkReregisterContext ReregisterContext(GetWorld()->Scene, MakeArrayView(ActorComponents.GetData(), ActorComponents.Num()));

			TArray<const UBlueprintGeneratedClass*> ParentBPClassStack;
			UBlueprintGeneratedClass::GetGeneratedClassesHierarchy(GetClass(), ParentBPClassStack);
			for (const UBlueprintGeneratedClass* BPClass : ParentBPClassStack)
			{
				if (BPClass->SimpleConstructionScript)
				{
					USimpleConstructionScript* SCS = BPClass->SimpleConstructionScript;
					ReregisterContext.AddSimpleConstructionScript(SCS);
				}
			}

			FNavigationLockContext NavLock(GetWorld(), ENavigationLockReason::AllowUnregister);
			RerunConstructionScripts();
			// Construction scripts can have all manner of side effects, including creation of render proxies, unregistering / destruction of components
			// Remove any static mesh components that are no longer valid for proxy registration after RerunConstructionScripts
			ReregisterContext.SanitizeMeshComponents();
		}
	}

	if (!FLevelUtils::IsMovingLevel())
	{
		GEngine->BroadcastOnActorMoving(this);
	}

	if ( bFinished )
	{
		UWorld* World = GetWorld();

		World->UpdateCullDistanceVolumes(this);
		World->bAreConstraintsDirty = true;

		FEditorSupportDelegates::RefreshPropertyWindows.Broadcast();

		// Let other systems know that an actor was moved
		GEngine->BroadcastOnActorMoved( this );

		FEditorSupportDelegates::UpdateUI.Broadcast();
	}

	// If the root component was not just recreated by the construction script - call PostEditComponentMove on it
	if(RootComponent != NULL && !RootComponent->IsCreatedByConstructionScript())
	{
		RootComponent->PostEditComponentMove(bFinished);
	}

	if (bFinished)
	{
		FNavigationSystem::OnPostEditActorMove(*this);
	}
}

bool AActor::ReregisterComponentsWhenModified() const
{
	// For child actors, redirect to the parent's owner (we do the same in RerunConstructionScripts).
	if (const AActor* ParentActor = GetParentActor())
	{
		return ParentActor->ReregisterComponentsWhenModified();
	}

	return !bActorIsBeingConstructed && !IsTemplate() && !GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor) && GetWorld() != nullptr;
}

void AActor::DebugShowComponentHierarchy(  const TCHAR* Info, bool bShowPosition )
{	
	TArray<AActor*> ParentedActors;
	GetAttachedActors( ParentedActors );
	if( Info  )
	{
		UE_LOGF( LogActor, Warning, "--%ls--", Info );
	}
	else
	{
		UE_LOGF( LogActor, Warning, "--------------------------------------------------" );
	}
	UE_LOGF( LogActor, Warning, "--------------------------------------------------" );
	UE_LOGF( LogActor, Warning, "Actor [%p] (%ls)", this, *GetFName().ToString() );
	USceneComponent* SceneComp = GetRootComponent();
	if( SceneComp )
	{
		int32 NestLevel = 0;
		DebugShowOneComponentHierarchy( SceneComp, NestLevel, bShowPosition );			
	}
	else
	{
		UE_LOGF( LogActor, Warning, "Actor has no root." );		
	}
	UE_LOGF( LogActor, Warning, "--------------------------------------------------" );
}

void AActor::DebugShowOneComponentHierarchy( USceneComponent* SceneComp, int32& NestLevel, bool bShowPosition )
{
	FString Nest;
	for (int32 iNest = 0; iNest < NestLevel ; iNest++)
	{
		Nest = Nest + "---->";	
	}
	NestLevel++;
	FString PosString;
	if( bShowPosition )
	{
		FVector Posn = SceneComp->GetComponentTransform().GetLocation();
		//PosString = FString::Printf( TEXT("{R:%f,%f,%f- W:%f,%f,%f}"), SceneComp->RelativeLocation.X, SceneComp->RelativeLocation.Y, SceneComp->RelativeLocation.Z, Posn.X, Posn.Y, Posn.Z );
		PosString = FString::Printf( TEXT("{R:%f- W:%f}"), SceneComp->GetRelativeLocation().Z, Posn.Z );
	}
	else
	{
		PosString = "";
	}
	AActor* OwnerActor = SceneComp->GetOwner();
	if( OwnerActor )
	{
		UE_LOGF(LogActor, Warning, "%lsSceneComp [%p] (%ls) Owned by %ls %ls", *Nest, SceneComp, *SceneComp->GetFName().ToString(), *OwnerActor->GetFName().ToString(), *PosString );
	}
	else
	{
		UE_LOGF(LogActor, Warning, "%lsSceneComp [%p] (%ls) No Owner", *Nest, SceneComp, *SceneComp->GetFName().ToString() );
	}
	if( SceneComp->GetAttachParent())
	{
		if( bShowPosition )
		{
			FVector Posn = SceneComp->GetComponentTransform().GetLocation();
			//PosString = FString::Printf( TEXT("{R:%f,%f,%f- W:%f,%f,%f}"), SceneComp->RelativeLocation.X, SceneComp->RelativeLocation.Y, SceneComp->RelativeLocation.Z, Posn.X, Posn.Y, Posn.Z );
			PosString = FString::Printf( TEXT("{R:%f- W:%f}"), SceneComp->GetRelativeLocation().Z, Posn.Z );
		}
		else
		{
			PosString = "";
		}
		UE_LOGF(LogActor, Warning, "%lsAttachParent [%p] (%ls) %ls", *Nest, SceneComp->GetAttachParent(), *SceneComp->GetAttachParent()->GetFName().ToString(), *PosString );
	}
	else
	{
		UE_LOGF(LogActor, Warning, "%ls[NO PARENT]", *Nest );
	}

	if( SceneComp->GetAttachChildren().Num() != 0 )
	{
		for (USceneComponent* EachSceneComp : SceneComp->GetAttachChildren())
		{			
			DebugShowOneComponentHierarchy(EachSceneComp,NestLevel, bShowPosition );
		}
	}
	else
	{
		UE_LOGF(LogActor, Warning, "%ls[NO CHILDREN]", *Nest );
	}
}

FActorTransactionAnnotation::FDiffableComponentInfo::FDiffableComponentInfo(const UActorComponent* Component)
	: ComponentSourceInfo(Component)
	, DiffableComponent(UE::Transaction::DiffUtil::GetDiffableObject(Component, UE::Transaction::DiffUtil::FGetDiffableObjectOptions{ UE::Transaction::DiffUtil::EGetDiffableObjectMode::SerializeProperties }))
{
}

TSharedRef<FActorTransactionAnnotation> FActorTransactionAnnotation::Create()
{
	return MakeShareable(new FActorTransactionAnnotation());
}

TSharedRef<FActorTransactionAnnotation> FActorTransactionAnnotation::Create(const AActor* InActor, const bool InCacheRootComponentData)
{
	return MakeShareable(new FActorTransactionAnnotation(InActor, FComponentInstanceDataCache(InActor), InCacheRootComponentData));
}

TSharedPtr<FActorTransactionAnnotation> FActorTransactionAnnotation::CreateIfRequired(const AActor* InActor, const bool InCacheRootComponentData)
{
	// Don't create a transaction annotation for something that has no instance data, or a root component that's created by a construction script
	FComponentInstanceDataCache TempComponentInstanceData(InActor);
	if (!TempComponentInstanceData.HasInstanceData())
	{
		USceneComponent* ActorRootComponent = InActor->GetRootComponent();
		if (!InCacheRootComponentData || !ActorRootComponent || !ActorRootComponent->IsCreatedByConstructionScript())
		{
			return nullptr;
		}
	}

	return MakeShareable(new FActorTransactionAnnotation(InActor, MoveTemp(TempComponentInstanceData), InCacheRootComponentData));
}

FActorTransactionAnnotation::FActorTransactionAnnotation()
{
	ActorTransactionAnnotationData.bRootComponentDataCached = false;
}

FActorTransactionAnnotation::FActorTransactionAnnotation(const AActor* InActor, FComponentInstanceDataCache&& InComponentInstanceData, const bool InCacheRootComponentData)
{
	ActorTransactionAnnotationData.ComponentInstanceData = MoveTemp(InComponentInstanceData);
	ActorTransactionAnnotationData.Actor = InActor;

	USceneComponent* ActorRootComponent = InActor->GetRootComponent();
	if (InCacheRootComponentData && ActorRootComponent && ActorRootComponent->IsCreatedByConstructionScript())
	{
		ActorTransactionAnnotationData.bRootComponentDataCached = true;
		FActorRootComponentReconstructionData& RootComponentData = ActorTransactionAnnotationData.RootComponentData;
		RootComponentData.Transform = ActorRootComponent->GetComponentTransform();
		RootComponentData.Transform.SetTranslation(ActorRootComponent->GetComponentLocation()); // take into account any custom location
		RootComponentData.TransformRotationCache = ActorRootComponent->GetRelativeRotationCache();

		if (ActorRootComponent->GetAttachParent())
		{
			RootComponentData.AttachedParentInfo.Actor = ActorRootComponent->GetAttachParent()->GetOwner();
			RootComponentData.AttachedParentInfo.AttachParent = ActorRootComponent->GetAttachParent();
			RootComponentData.AttachedParentInfo.AttachParentName = ActorRootComponent->GetAttachParent()->GetFName();
			RootComponentData.AttachedParentInfo.SocketName = ActorRootComponent->GetAttachSocketName();
			RootComponentData.AttachedParentInfo.RelativeTransform = ActorRootComponent->GetRelativeTransform();
		}

		for (USceneComponent* AttachChild : ActorRootComponent->GetAttachChildren())
		{
			AActor* ChildOwner = (AttachChild ? AttachChild->GetOwner() : NULL);
			if (ChildOwner && ChildOwner != InActor)
			{
				// Save info about actor to reattach
				FActorRootComponentReconstructionData::FAttachedActorInfo Info;
				Info.Actor = ChildOwner;
				Info.SocketName = AttachChild->GetAttachSocketName();
				Info.RelativeTransform = AttachChild->GetRelativeTransform();
				RootComponentData.AttachedToInfo.Add(Info);
			}
		}
	}
	else
	{
		ActorTransactionAnnotationData.bRootComponentDataCached = false;
	}

	// This code also runs when reconstructing actors, so only cache the diff data when we're creating or applying a transaction
	if (GUndo || GIsTransacting)
	{
		TInlineComponentArray<UActorComponent*> Components;
		FComponentInstanceDataCache::GetComponentHierarchy(InActor, Components);

		const bool bIsChildActor = InActor->IsChildActor();
		for (UActorComponent* Component : Components)
		{
			if (Component && (bIsChildActor || Component->IsCreatedByConstructionScript()))
			{
				DiffableComponentInfos.Emplace(Component);
			}
		}
	}
}

void FActorTransactionAnnotation::AddReferencedObjects(FReferenceCollector& Collector)
{
	ActorTransactionAnnotationData.ComponentInstanceData.AddReferencedObjects(Collector);
}

void FActorTransactionAnnotation::Serialize(FArchive& Ar)
{
	Ar << ActorTransactionAnnotationData;
}

void FActorTransactionAnnotation::ComputeAdditionalObjectChanges(const ITransactionObjectAnnotation* OriginalAnnotation, TMap<UObject*, FTransactionObjectChange>& OutAdditionalObjectChanges)
{
	const FActorTransactionAnnotation* OriginalActorAnnotation = static_cast<const FActorTransactionAnnotation*>(OriginalAnnotation);

	struct FDiffableComponentPair
	{
		UActorComponent* CurrentComponent = nullptr;
		const UE::Transaction::FDiffableObject* OldDiffableComponent = nullptr;
		const UE::Transaction::FDiffableObject* NewDiffableComponent = nullptr;
	};

	TArray<FDiffableComponentPair, TInlineAllocator<NumInlinedActorComponents>> DiffableComponentPairs;
	if (const AActor* Actor = ActorTransactionAnnotationData.Actor.Get(/*bEvenIfPendingKill*/true))
	{
		TInlineComponentArray<UActorComponent*> Components;
		FComponentInstanceDataCache::GetComponentHierarchy(Actor, Components);

		if (Components.Num() > 0)
		{
			// Bitsets to avoid repeated testing of FDiffableComponentInfo that has already been matched to a known component
			TBitArray<> OriginalDiffableComponentInfosToConsider(true, OriginalActorAnnotation ? OriginalActorAnnotation->DiffableComponentInfos.Num() : 0);
			TBitArray<> DiffableComponentInfosToConsider(true, DiffableComponentInfos.Num());

			const bool bIsChildActor = Actor->IsChildActor();
			DiffableComponentPairs.Reserve(Components.Num());
			for (UActorComponent* Component : Components)
			{
				if (Component && (bIsChildActor || Component->IsCreatedByConstructionScript()))
				{
					auto FindDiffableComponent = [Component](const TArray<FDiffableComponentInfo>& DiffableComponentInfosToSearch, TBitArray<>& DiffableComponentInfosBitset) -> const UE::Transaction::FDiffableObject*
					{
						const UObject* ComponentTemplate = Component->GetArchetype();
						for (TConstSetBitIterator<> BitsetIt(DiffableComponentInfosBitset); BitsetIt; ++BitsetIt)
						{
							const FDiffableComponentInfo& PotentialDiffableComponentInfo = DiffableComponentInfosToSearch[BitsetIt.GetIndex()];
							if (PotentialDiffableComponentInfo.ComponentSourceInfo.MatchesComponent(Component, ComponentTemplate))
							{
								DiffableComponentInfosBitset[BitsetIt.GetIndex()] = false;
								return &PotentialDiffableComponentInfo.DiffableComponent;
							}
						}
						return nullptr;
					};

					FDiffableComponentPair& DiffableComponentPair = DiffableComponentPairs.AddDefaulted_GetRef();
					DiffableComponentPair.CurrentComponent = Component;
					DiffableComponentPair.OldDiffableComponent = OriginalActorAnnotation ? FindDiffableComponent(OriginalActorAnnotation->DiffableComponentInfos, OriginalDiffableComponentInfosToConsider) : nullptr;
					DiffableComponentPair.NewDiffableComponent = FindDiffableComponent(DiffableComponentInfos, DiffableComponentInfosToConsider);
				}
			}
		}
	}

	if (DiffableComponentPairs.Num() > 0)
	{
		UE::Transaction::DiffUtil::FDiffableObjectArchetypeCache ArchetypeCache;
		
		auto ComputeAdditionalObjectChange = [&OutAdditionalObjectChanges, &ArchetypeCache](const UE::Transaction::FDiffableObject& OldDiffableComponent, const UE::Transaction::FDiffableObject& NewDiffableComponent, UActorComponent* CurrentComponent)
		{
			UE::Transaction::DiffUtil::FGenerateObjectDiffOptions DiffOptions;
			DiffOptions.ArchetypeOptions.ObjectSerializationMode = UE::Transaction::DiffUtil::EGetDiffableObjectMode::SerializeProperties;
			{
				TSet<const FProperty*> PropertiesToSkip;

				if (!ActorEditorSettings::bIncludeSCSModifiedPropertiesInDiff)
				{
					// Skip properties modified during construction when calculating the diff
					CurrentComponent->GetUCSModifiedProperties(PropertiesToSkip);

					// If this is the owning Actor's root scene component, always include relative transform properties as GetUCSModifiedProperties incorrectly considers them modified (due to changing during placement)
					if (CurrentComponent->IsA<USceneComponent>())
					{
						const AActor* ComponentOwner = CurrentComponent->GetOwner();
						if (ComponentOwner && ComponentOwner->GetRootComponent() == CurrentComponent)
						{
							PropertiesToSkip.Remove(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName()));
							PropertiesToSkip.Remove(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName()));
							PropertiesToSkip.Remove(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName()));
						}
					}
				}

				// Always skip the construction information for managed components, as this is managed by the construction script
				if (CurrentComponent->CreationMethod != EComponentCreationMethod::Instance)
				{
					static const FName NAME_CreationMethod = "CreationMethod";
					static const FName NAME_UCSSerializationIndex = "UCSSerializationIndex";

					PropertiesToSkip.Add(FindFProperty<FProperty>(UActorComponent::StaticClass(), NAME_CreationMethod));
					PropertiesToSkip.Add(FindFProperty<FProperty>(UActorComponent::StaticClass(), NAME_UCSSerializationIndex));
				}

				DiffOptions.ShouldSkipProperty = [CurrentComponent, PropertiesToSkip = MoveTemp(PropertiesToSkip)](FName PropertyName) -> bool
				{
					if (const FProperty* Property = FindFProperty<FProperty>(CurrentComponent->GetClass(), PropertyName))
					{
						return Property->IsA<FMulticastDelegateProperty>()
							|| PropertiesToSkip.Contains(Property);
					}
					return false;
				};
			}

			FTransactionObjectDeltaChange ComponentDeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(OldDiffableComponent, NewDiffableComponent, DiffOptions, &ArchetypeCache);
			if (ComponentDeltaChange.HasChanged())
			{
				if (FTransactionObjectChange* ExistingComponentChange = OutAdditionalObjectChanges.Find(CurrentComponent))
				{
					ExistingComponentChange->DeltaChange.Merge(ComponentDeltaChange);
				}
				else
				{
					OutAdditionalObjectChanges.Add(CurrentComponent, FTransactionObjectChange{ OldDiffableComponent.ObjectInfo, OldDiffableComponent.ObjectFlags, MoveTemp(ComponentDeltaChange) });
				}
			}
		};

		for (const FDiffableComponentPair& DiffableComponentPair : DiffableComponentPairs)
		{
			check(DiffableComponentPair.CurrentComponent);

			if (DiffableComponentPair.OldDiffableComponent && DiffableComponentPair.NewDiffableComponent)
			{
				// Component already existed; diff against its previous state
				ComputeAdditionalObjectChange(*DiffableComponentPair.OldDiffableComponent, *DiffableComponentPair.NewDiffableComponent, DiffableComponentPair.CurrentComponent);
			}
			else if (DiffableComponentPair.OldDiffableComponent && !DiffableComponentPair.NewDiffableComponent)
			{
				// Component is deleted; just mark it as pending kill
				UE::Transaction::FDiffableObject FakeDiffableComponent;
				FakeDiffableComponent.SetObject(DiffableComponentPair.CurrentComponent);
				FakeDiffableComponent.ObjectInfo.bIsPendingKill = true;

				ComputeAdditionalObjectChange(*DiffableComponentPair.OldDiffableComponent, FakeDiffableComponent, DiffableComponentPair.CurrentComponent);
			}
			else if (DiffableComponentPair.NewDiffableComponent)
			{
				check(!DiffableComponentPair.OldDiffableComponent);

				// Component is newly added; diff against its archetype
				UE::Transaction::FDiffableObject FakeDiffableComponent;
				FakeDiffableComponent.SetObject(DiffableComponentPair.CurrentComponent);
				FakeDiffableComponent.ObjectInfo.bIsPendingKill = true;

				ComputeAdditionalObjectChange(FakeDiffableComponent, *DiffableComponentPair.NewDiffableComponent, DiffableComponentPair.CurrentComponent);
			}
		}
	}
}

bool FActorTransactionAnnotation::HasInstanceData() const
{
	return (ActorTransactionAnnotationData.bRootComponentDataCached || ActorTransactionAnnotationData.ComponentInstanceData.HasInstanceData());
}

TSharedPtr<ITransactionObjectAnnotation> AActor::FactoryTransactionAnnotation(const ETransactionAnnotationCreationMode InCreationMode) const
{
	if (InCreationMode == ETransactionAnnotationCreationMode::DefaultInstance)
	{
		return FActorTransactionAnnotation::Create();
	}

	if (CurrentTransactionAnnotation.IsValid())
	{
		return CurrentTransactionAnnotation;
	}

	return FActorTransactionAnnotation::CreateIfRequired(this);
}

void AActor::PreEditUndo()
{
	// Check if this Actor needs to be re-instanced
	UClass* OldClass = GetClass();
	UClass* NewClass = OldClass->GetAuthoritativeClass();
	if (NewClass != OldClass)
	{
		// Empty the OwnedComponents array, it's filled with invalid information
		OwnedComponents.Empty();
	}

	IntermediateOwner = Owner;
	// Since child actor components will rebuild themselves get rid of the Actor before we make changes
	TInlineComponentArray<UChildActorComponent*> ChildActorComponents;
	GetComponents(ChildActorComponents);

	for (UChildActorComponent* ChildActorComponent : ChildActorComponents)
	{
		if (ChildActorComponent->IsCreatedByConstructionScript())
		{
			ChildActorComponent->DestroyChildActor();
		}
	}

	// let navigation system know to not care about this actor anymore
	FNavigationSystem::RemoveActorData(*this);

	Super::PreEditUndo();
}

bool AActor::InternalPostEditUndo()
{
	if (IntermediateOwner != Owner)
	{
		AActor* TempOwner = Owner;
		Owner = IntermediateOwner.Get();
		SetOwner(TempOwner);
	}
	IntermediateOwner = nullptr;

	// Check if this Actor needs to be re-instanced
	UClass* OldClass = GetClass();
	if (OldClass->HasAnyClassFlags(CLASS_NewerVersionExists))
	{
		UClass* NewClass = OldClass->GetAuthoritativeClass();
		if (!ensure(NewClass != OldClass))
		{
			UE_LOGF(LogActor, Warning, "WARNING: %ls is out of date and is the same as its AuthoritativeClass during PostEditUndo!", *OldClass->GetName());
		};

		// Early exit, letting anything more occur would be invalid due to the REINST_ class
		return false;
	}

	// Notify LevelBounds actor that level bounding box might be changed
	if (!IsTemplate())
	{
		if (ULevel* Level = GetLevel())
		{
			Level->MarkLevelBoundsDirty();
		}
	}

	// Restore OwnedComponents array
	if (IsValidChecked(this))
	{
		ResetOwnedComponents();

		// BP created components are not serialized, so this should be cleared and will be filled in as the construction scripts are run
		BlueprintCreatedComponents.Reset();

		// notify navigation system
		FNavigationSystem::UpdateActorAndComponentData(*this);
		UEngineElementsLibrary::RegisterActorElement(this);
	}
	else
	{
		FNavigationSystem::RemoveActorData(*this);
		UEngineElementsLibrary::UnregisterActorElement(this);
	}

	// This is a normal undo, so call super
	return true;
}

void AActor::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);
	if (TransactionEvent.HasOuterChange())
	{
		GEngine->BroadcastLevelActorOuterChanged(this, StaticFindObject(ULevel::StaticClass(), nullptr, *TransactionEvent.GetOriginalObjectOuterPathName().ToString()));
	}
}

void AActor::PostEditUndo()
{
	if (InternalPostEditUndo())
	{
		Super::PostEditUndo();
	}

	// Do not immediately update all primitive scene infos for brush actor
	// undo/redo transactions since they require the render thread to wait until
	// after the transactions are processed to guarantee that the model data
	// is safe to access.
	UWorld* World = GetWorld();
	if (World && World->Scene && !FActorEditorUtils::IsABrush(this))
	{
		UE::RenderCommandPipe::FSyncScope SyncScope;

		ENQUEUE_RENDER_COMMAND(UpdateAllPrimitiveSceneInfosCmd)([Scene = World->Scene](FRHICommandListImmediate& RHICmdList) {
			Scene->UpdateAllPrimitiveSceneInfos(RHICmdList);
		});
	}
}

void AActor::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)
{
	CurrentTransactionAnnotation = StaticCastSharedPtr<FActorTransactionAnnotation>(TransactionAnnotation);

	if (InternalPostEditUndo())
	{
		Super::PostEditUndo(TransactionAnnotation);
	}
}

// @todo: Remove this hack once we have decided on the scaling method to use.
bool AActor::bUsePercentageBasedScaling = false;

void AActor::EditorApplyTranslation(const FVector& DeltaTranslation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	if( RootComponent != NULL )
	{
		FTransform NewTransform = GetRootComponent()->GetComponentTransform();
		NewTransform.SetTranslation(NewTransform.GetTranslation() + DeltaTranslation);
		GetRootComponent()->SetWorldTransform(NewTransform);
	}
	else
	{
		UE_LOGF(LogActor, Warning, "WARNING: EditorApplyTranslation %ls has no root component", *GetName() );
	}
}

void AActor::EditorApplyRotation(const FRotator& DeltaRotation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	if( RootComponent != NULL )
	{
		FRotator Rot = RootComponent->GetAttachParent() != NULL ? GetActorRotation() : RootComponent->GetRelativeRotation();
		FRotator ActorRotWind, ActorRotRem;
		Rot.GetWindingAndRemainder(ActorRotWind, ActorRotRem);
		const FQuat ActorQ = ActorRotRem.Quaternion();
		const FQuat DeltaQ = DeltaRotation.Quaternion();

		FRotator NewActorRotRem;
		if(RootComponent->GetAttachParent() != NULL )
		{
			//first we get the new rotation in relative space.
			const FQuat ResultQ = DeltaQ * ActorQ;
			NewActorRotRem = FRotator(ResultQ);
			FRotator DeltaRot = NewActorRotRem - ActorRotRem;
			FRotator NewRotation = Rot + DeltaRot;
			FQuat NewRelRotation = NewRotation.Quaternion();
			NewRelRotation = RootComponent->GetRelativeRotationFromWorld(NewRelRotation);
			NewActorRotRem = FRotator(NewRelRotation);
			//now we need to get current relative rotation to find the diff
			Rot = RootComponent->GetRelativeRotation();
			Rot.GetWindingAndRemainder(ActorRotWind, ActorRotRem);
		}
		else
		{
			const FQuat ResultQ = DeltaQ * ActorQ;
			NewActorRotRem = FRotator(ResultQ);
		}

		ActorRotRem.SetClosestToMe(NewActorRotRem);
		FRotator DeltaRot = NewActorRotRem - ActorRotRem;
		DeltaRot.Normalize();
		RootComponent->SetRelativeRotationExact( Rot + DeltaRot );
	}
	else
	{
		UE_LOGF(LogActor, Warning, "WARNING: EditorApplyRotation %ls has no root component", *GetName() );
	}
}


void AActor::EditorApplyScale( const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown )
{
	if( RootComponent != NULL )
	{
		const FVector CurrentScale = GetRootComponent()->GetRelativeScale3D();

		// @todo: Remove this hack once we have decided on the scaling method to use.
		FVector ScaleToApply;

		if( AActor::bUsePercentageBasedScaling )
		{
			ScaleToApply = CurrentScale * (FVector(1.0f) + DeltaScale);
		}
		else
		{
			ScaleToApply = CurrentScale + DeltaScale;
		}

		GetRootComponent()->SetRelativeScale3D(ScaleToApply);

		if (PivotLocation)
		{
			const FVector CurrentScaleSafe(CurrentScale.X ? CurrentScale.X : 1.0f,
										   CurrentScale.Y ? CurrentScale.Y : 1.0f,
										   CurrentScale.Z ? CurrentScale.Z : 1.0f);

			const FRotator ActorRotation = GetActorRotation();
			const FVector WorldDelta = GetActorLocation() - (*PivotLocation);
			const FVector LocalDelta = (ActorRotation.GetInverse()).RotateVector(WorldDelta);
			const FVector LocalScaledDelta = LocalDelta * (ScaleToApply / CurrentScaleSafe);
			const FVector WorldScaledDelta = ActorRotation.RotateVector(LocalScaledDelta);

			GetRootComponent()->SetWorldLocation(WorldScaledDelta + (*PivotLocation));
		}
	}
	else
	{
		UE_LOGF(LogActor, Warning, "WARNING: EditorApplyTranslation %ls has no root component", *GetName() );
	}

	FEditorSupportDelegates::UpdateUI.Broadcast();
}


void AActor::EditorApplyMirror(const FVector& MirrorScale, const FVector& PivotLocation)
{
	const FRotationMatrix TempRot( GetActorRotation() );
	const FVector New0( TempRot.GetScaledAxis( EAxis::X ) * MirrorScale );
	const FVector New1( TempRot.GetScaledAxis( EAxis::Y ) * MirrorScale );
	const FVector New2( TempRot.GetScaledAxis( EAxis::Z ) * MirrorScale );
	// Revert the handedness of the rotation, but make up for it in the scaling.
	// Arbitrarily choose the X axis to remain fixed.
	const FMatrix NewRot( -New0, New1, New2, FVector::ZeroVector );

	if( RootComponent != NULL )
	{
		GetRootComponent()->SetWorldRotation(NewRot.Rotator());
		FVector Loc = GetActorLocation();
		Loc -= PivotLocation;
		Loc *= MirrorScale;
		Loc += PivotLocation;
		GetRootComponent()->SetWorldLocation(Loc);

		FVector Scale3D = GetRootComponent()->GetRelativeScale3D();
		Scale3D.X = -Scale3D.X;
		GetRootComponent()->SetRelativeScale3D(Scale3D);
	}
	else
	{
		UE_LOGF(LogActor, Warning, "WARNING: EditorApplyMirror %ls has no root component", *GetName() );
	}
}

void AActor::EditorGetUnderlyingActors(TSet<AActor*>& OutUnderlyingActors) const
{
	TInlineComponentArray<UChildActorComponent*> ChildActorComponents;
	GetComponents(ChildActorComponents);

	OutUnderlyingActors.Reserve(OutUnderlyingActors.Num() + ChildActorComponents.Num());
	
	for (UChildActorComponent* ChildActorComponent : ChildActorComponents)
	{
		if (AActor* ChildActor = ChildActorComponent->GetChildActor())
		{
			bool bAlreadySet = false;
			OutUnderlyingActors.Add(ChildActor, &bAlreadySet);
			if (!bAlreadySet)
			{
				ChildActor->EditorGetUnderlyingActors(OutUnderlyingActors);
			}
		}
	}
}

bool AActor::IsHiddenEd() const
{
	// If any of the standard hide flags are set, return true
	if( bHiddenEdLayer || !bEditable || ( GIsEditor && ( IsTemporarilyHiddenInEditor() || bHiddenEdLevel ) ) )
	{
		return true;
	}
	// Otherwise, it's visible
	return false;
}

void AActor::SetIsTemporarilyHiddenInEditor( bool bIsHidden )
{
	if( bHiddenEdTemporary != bIsHidden )
	{
		bHiddenEdTemporary = bIsHidden;
		MarkComponentsRenderStateDirty();

		// When updating the visibility we need to add the sync tag to the row in TEDS to ensure the visibility is synced in TEDS properly
		using namespace UE::Editor::DataStorage;
		ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		ICompatibilityProvider* StorageCompatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);

		if (Storage && StorageCompatibility)
		{
			RowHandle ActorRow = StorageCompatibility->FindRowWithCompatibleObject(this);

			if (Storage->IsRowAvailable(ActorRow))
			{
				Storage->AddColumn<FTypedElementSyncFromWorldTag>(ActorRow);
			}
		}
	}
}

bool AActor::SetIsHiddenEdLayer(bool bIsHiddenEdLayer)
{
	if (bHiddenEdLayer != bIsHiddenEdLayer)
	{
		bHiddenEdLayer = bIsHiddenEdLayer;
		MarkComponentsRenderStateDirty();
		return true;
	}
	return false;
}

bool AActor::SupportsLayers() const
{
	const bool bIsHidden = (GetClass()->GetDefaultObject<AActor>()->bHiddenEd == true);
	const bool bIsInEditorWorld = (GetWorld()->WorldType == EWorldType::Editor);
	const bool bIsPartitionedActor = GetLevel()->bIsPartitioned;
	const bool bIsValid = !bIsHidden && bIsInEditorWorld && !bIsPartitionedActor;

	if (bIsValid)
	{
		// Actors part of Level Instance are not valid for layers
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			if (ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(this))
			{
				return false;
			}
		}
	}

	return bIsValid;
}

bool AActor::IsForceExternalActorLevelReferenceForPIE() const
{
	if (const AActor* ParentActor = GetParentActor())
	{
		return ParentActor->IsForceExternalActorLevelReferenceForPIE();
	}

	return bForceExternalActorLevelReferenceForPIE;
}

bool AActor::IsEditable() const
{
	return bEditable;
}

bool AActor::IsSelectable() const
{
	return true;
}

bool AActor::IsListedInSceneOutliner() const
{
	return bListedInSceneOutliner;
}

bool AActor::EditorCanAttachTo(const AActor* InParent, FText& OutReason) const
{
	return true;
}

bool AActor::EditorCanAttachFrom(const AActor* InChild, FText& OutReason) const
{
	return true;
}

AActor* AActor::GetSceneOutlinerParent() const
{
	return GetAttachParentActor();
}

#if !UE_STRIP_DEPRECATED_PROPERTIES 
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UPackage* AActor::GetSceneOutlinerPackage() const
{
	return GetSceneOutlinerItemPackage();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

UPackage* AActor::GetSceneOutlinerItemPackage() const
{
	return GetExternalPackage();
}

UPackage* AActor::GetSceneOutlinerTopParentPackage() const
{
	return GetPackage();
}

class UHLODLayer* AActor::GetHLODLayer() const
{
	return HLODLayer;
}

void AActor::SetHLODLayer(class UHLODLayer* InHLODLayer)
{
	HLODLayer = InHLODLayer;
}

bool AActor::IsMainWorldOnly() const
{
	if (bIsMainWorldOnly)
	{
		return true;
	}
	
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return ActorTypeIsMainWorldOnly();
	}
	else
	{
		return CastChecked<AActor>(GetClass()->GetDefaultObject())->IsMainWorldOnly();
	}
}

void AActor::SetPackageExternal(bool bExternal, bool bShouldDirty, UPackage* ActorExternalPackage)
{
	check(bExternal || !ActorExternalPackage);

	// @todo_ow: Call FExternalPackageHelper::SetPackagingMode and keep calling the actor specific code here (components). 
	//           The only missing part is GetExternalObjectsPath defaulting to a different folder than the one used by external actors.
	if (bExternal == IsPackageExternal())
	{
		return;
	}

    // Mark the current actor & package as dirty
	Modify(bShouldDirty);

	if (bExternal)
	{
		UPackage* LevelPackage = GetLevel()->GetPackage();
		UPackage* ActorPackage = ActorExternalPackage ? ActorExternalPackage : ULevel::CreateActorPackage(LevelPackage, GetLevel()->GetActorPackagingScheme(), GetPathName(), this);
		// Cleanup the package from its UDeletedObjectPlaceholder if any (CreateActorPackage can return an existing package)
		UDeletedObjectPlaceholder::RemoveFromPackage(ActorPackage);
		SetExternalPackage(ActorPackage);
	}
	else
	{
		const bool bWasMainActorInPackage = IsMainPackageActor();
		UPackage* ActorPackage = GetExternalPackage();
		// Detach the linker exports so it doesn't resolve to this actor anymore
		ResetLinkerExports(ActorPackage);
		SetExternalPackage(nullptr);

		// If the old external package is empty, create a dummy object outered to the level part of the original external package
		// so that ULevel::CleanupLevel will be able to visit & process this empty external package.
		// This object will allow the save package dialog to display the label of the previously removed actor (see FAssetTypeActions_DeletedObjectPlaceholder::GetObjectDisplayName)
		if (!ActorPackage->HasAnyPackageFlags(PKG_NewlyCreated) && bWasMainActorInPackage && !AActor::FindActorInPackage(ActorPackage))
		{
			UDeletedObjectPlaceholder::Create(GetLevel(), ActorPackage, this);
		}
	}

	for (UActorComponent* ActorComponent : GetComponents())
	{
		if (ActorComponent && ActorComponent->IsRegistered())
		{
			ActorComponent->SetPackageExternal(bExternal, bShouldDirty);
		}
	}

	OnPackagingModeChanged.Broadcast(this, bExternal);
	
	// Mark the new actor package dirty
	MarkPackageDirty();
}

bool AActor::IsLockLocation() const
{
	if (IsInLevelInstance() && !IsInAnyEditLevelInstance())
	{
		return true;
	}

	return bLockLocation;
}

void AActor::OnPlayFromHere()
{
	check(bCanPlayFromHere);
}

TUniquePtr<FWorldPartitionActorDesc> AActor::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FWorldPartitionActorDesc());
}

TUniquePtr<FWorldPartitionActorDesc> AActor::CreateActorDesc() const
{
	TUniquePtr<FWorldPartitionActorDesc> ActorDesc(CreateClassActorDesc());
		
	ActorDesc->Init(this);
	
	return ActorDesc;
}

void AActor::GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const
{
	ForEachComponent<UActorComponent>(false, [&PropertyPairsMap](UActorComponent* Component)
	{
		Component->GetActorDescProperties(PropertyPairsMap);
	});
}

TUniquePtr<class FWorldPartitionActorDesc> AActor::StaticCreateClassActorDesc(const TSubclassOf<AActor>& ActorClass)
{
	return CastChecked<AActor>(ActorClass->GetDefaultObject())->CreateClassActorDesc();
}

FString AActor::GetDefaultActorLabel() const
{
	UClass* ActorClass = GetClass();

	FString DefaultActorLabel = ActorClass->GetName();

	// Strip off the ugly "_C" suffix for Blueprint class actor instances
	if (Cast<UBlueprint>(ActorClass->ClassGeneratedBy))
	{
		DefaultActorLabel.RemoveFromEnd(TEXT("_C"), ESearchCase::CaseSensitive);
	}

	return DefaultActorLabel;
}

const FString& AActor::GetActorLabel(bool bCreateIfNone) const
{
	// If the label string is empty then we'll use the default actor label (usually the actor's class name.)
	// We actually cache the default name into our ActorLabel property.  This will be saved out with the
	// actor if the actor gets saved.  The reasons we like caching the name here is:
	//
	//		a) We can return it by const&	(performance)
	//		b) Calling GetDefaultActorLabel() is slow because of FName stuff  (performance)
	//		c) If needed, we could always empty the ActorLabel string if it matched the default
	//
	// Remember, ActorLabel is currently an editor-only property.

	if( ActorLabel.IsEmpty() && bCreateIfNone )
	{
		FString DefaultActorLabel = GetDefaultActorLabel();

		// We want the actor's label to be initially unique, if possible, so we'll use the number of the
		// actor's FName when creating the initially.  It doesn't actually *need* to be unique, this is just
		// an easy way to tell actors apart when observing them in a list.  The user can always go and rename
		// these labels such that they're no longer unique.
		if (!FActorSpawnUtils::IsGloballyUniqueName(GetFName()))
		{
			// Don't bother adding a suffix for number '0'
			if (const int32 NameNumber = GetFName().GetNumber(); NameNumber != NAME_NO_NUMBER_INTERNAL)
			{
				DefaultActorLabel.AppendInt(NAME_INTERNAL_TO_EXTERNAL(NameNumber));
			}
		}

		// Remember, there could already be an actor with the same label in the level.  But that's OK, because
		// actor labels aren't supposed to be unique.  We just try to make them unique initially to help
		// disambiguate when opening up a new level and there are hundreds of actors of the same type.
		AActor* MutableThis = const_cast<AActor*>(this);
		MutableThis->ActorLabel = MoveTemp(DefaultActorLabel);
		FCoreDelegates::OnActorLabelChanged.Broadcast(MutableThis);
	}

	return ActorLabel;
}

void AActor::SetActorLabel(const FString& NewActorLabelDirty, bool bMarkDirty)
{
	// Clean up the incoming string a bit
	FString NewActorLabel = NewActorLabelDirty.TrimStartAndEnd();

	// Validate incoming string before proceeding
	FText OutErrorMessage;
	if (!FActorEditorUtils::ValidateActorName(FText::FromString(NewActorLabel), OutErrorMessage))
	{
		//Invalid actor name
		UE_LOGF(LogActor, Warning, "SetActorLabel failed: %ls", *OutErrorMessage.ToString());
	}
	// Only update the label if it has actually changed. We pass 'false' to GetActorLabel to prevent it
	//    creating a redundant name and broadcasting PostEditChangeProperty and OnActorLabelChanged unnecessarily
	else if (FCString::Strcmp(*NewActorLabel, *GetActorLabel(false)) != 0)
	{
		// Store new label
		Modify(bMarkDirty);
		ActorLabel = MoveTemp(NewActorLabel);

		FPropertyChangedEvent PropertyEvent(FindFProperty<FProperty>(AActor::StaticClass(), "ActorLabel"));
		PostEditChangeProperty(PropertyEvent);

		FCoreDelegates::OnActorLabelChanged.Broadcast(this);
	}
}

void AActor::SetInitialActorLabel(FStringView InitialLabel)
{
	if (!ensureMsgf(bHasFinishedSpawning == false, TEXT("It is invalid to call AActor::SetInitialActorLabel after it has finished spawning.")))
	{
		return;
	}

	FText OutErrorMessage;
	if (!FActorEditorUtils::ValidateActorLabel(InitialLabel, OutErrorMessage))
	{
		UE_LOGF(LogActor, Warning, "SetActorLabel failed: %ls", *OutErrorMessage.ToString());
	}
	else
	{
		ActorLabel = FString(InitialLabel);
	}
}

bool AActor::IsActorLabelEditable() const
{
	if (IsInLevelInstance() && !IsInEditLevelInstance())
	{
		return false;
	}

	return bActorLabelEditable;
}

void AActor::ClearActorLabel()
{
	if (!ActorLabel.IsEmpty())
	{
		ActorLabel.Reset();
		FCoreDelegates::OnActorLabelChanged.Broadcast(this);
	}
}

FFolder AActor::GetFolder() const
{
	// Favor building FFolder using guid (if available).
	// The reason is that a lot of calling functions will try resolving at some point the UActorFolder* from the folder
	// and the guid implementation is much faster.
	if (GetWorld() && GetFolderGuid().IsValid())
	{
		if (UActorFolder* ActorFolder = GetActorFolder())
		{
			return ActorFolder->GetFolder();
		}
	}
	return FFolder(GetFolderRootObject(), GetFolderPath());
}

FFolder::FRootObject AActor::GetFolderRootObject() const
{
	if (GetWorld())
	{
		return FFolder::GetOptionalFolderRootObject(GetLevel()).Get(FFolder::GetInvalidRootObject());
	}
	return FFolder::GetInvalidRootObject();
}

static bool IsUsingActorFolders(const AActor* InActor)
{
	return InActor && InActor->GetLevel() && InActor->GetLevel()->IsUsingActorFolders();
}

bool AActor::IsActorFolderValid() const
{
	return !IsUsingActorFolders(this) || (FolderPath.IsNone() && !FolderGuid.IsValid()) || GetActorFolder();
}

bool AActor::CreateOrUpdateActorFolder()
{
	check(GetLevel());
	check(IsUsingActorFolders(this));

	// First time this function is called, FolderPath can be valid and FolderGuid is invalid.
	if (FolderPath.IsNone() && !FolderGuid.IsValid())
	{
		// Nothing to do
		return true;
	}

	// Remap deleted folder or fixup invalid guid
	UActorFolder* ActorFolder = nullptr;
	if (FolderGuid.IsValid())
	{
		check(FolderPath.IsNone());
		ActorFolder = GetActorFolder(/*bSkipDeleted*/false);
		if (!ActorFolder || ActorFolder->IsMarkedAsDeleted())
		{
			FixupActorFolder();
			check(IsActorFolderValid());
			return true;
		}
	}

	// If not found, create actor folder using FolderPath
	if (!ActorFolder)
	{
		check(!FolderPath.IsNone());
		ActorFolder = FWorldPersistentFolders::GetActorFolder(FFolder(GetFolderRootObject(), FolderPath), GetWorld(), /*bAllowCreate*/ true);
	}

	// At this point, actor folder should always be valid
	if (ensure(ActorFolder))
	{
		SetFolderGuidInternal(ActorFolder ? ActorFolder->GetGuid() : FGuid());

		// Make sure actor folder is in the correct packaging mode
		ActorFolder->SetPackageExternal(GetLevel()->IsUsingExternalObjects());
	}
	return IsActorFolderValid();
}

UActorFolder* AActor::GetActorFolder(bool bSkipDeleted) const
{
	UActorFolder* ActorFolder = nullptr;
	if (ULevel* Level = GetLevel())
	{
	if (FolderGuid.IsValid())
	{
			ActorFolder = Level->GetActorFolder(FolderGuid, bSkipDeleted);
	}
	else if (!FolderPath.IsNone())
	{
			ActorFolder = Level->GetActorFolder(FolderPath);
	}
	}
	return ActorFolder;
}

void AActor::FixupActorFolder()
{
	check(GetLevel());

	if (!IsUsingActorFolders(this))
	{
		if (FolderGuid.IsValid())
		{
			UE_LOGF(LogActor, Warning, "Actor folder %ls for actor %ls encountered when not using actor folders", *FolderGuid.ToString(), *GetName());
			FolderGuid = FGuid();
		}
	}
	else
	{
		// First detect and fixup reference to deleted actor folders
		UActorFolder* ActorFolder = GetActorFolder(/*bSkipDeleted*/ false);
		if (ActorFolder)
		{
			// Remap to skip deleted actor folder
			if (ActorFolder->IsMarkedAsDeleted())
			{
				ActorFolder = ActorFolder->GetParent();
				SetFolderGuidInternal(ActorFolder ? ActorFolder->GetGuid() : FGuid(), /*bBroadcastChange*/ false);
			}
			// We found actor folder using its path, update actor folder guid
			else if (!FolderPath.IsNone())
			{
				SetFolderGuidInternal(ActorFolder ? ActorFolder->GetGuid() : FGuid(), /*bBroadcastChange*/ false);
			}
		}

		// If still invalid, fallback to root
		if (!IsActorFolderValid())
		{
			// Here we don't warn anymore since there's a supported workflow where we allow to delete actor folders 
			// even when referenced by actors (i.e. when the end result remains the root)
			SetFolderGuidInternal(FGuid(), /*bBroadcastChange*/ false);
		}

		if (!FolderPath.IsNone())
		{
			UE_LOGF(LogActor, Warning, "Actor folder path %ls for actor %ls encountered when using actor folders", *FolderPath.ToString(), *GetName());
			FolderPath = NAME_None;
		}
	}
}

FGuid AActor::GetFolderGuid(bool bDirectAccess) const
{
	return bDirectAccess || IsUsingActorFolders(this) ? FolderGuid : FGuid();
}

FName AActor::GetFolderPath() const
{
	static const FName RootPath = FFolder::GetEmptyPath();
	UWorld* World = GetWorld();
	if (World && !FFolder::GetOptionalFolderRootObject(GetLevel()))
	{
		return RootPath;
	}
	if (IsUsingActorFolders(this))
	{
		if (UActorFolder* ActorFolder = World ? GetActorFolder() : nullptr)
		{
			return ActorFolder->GetPath();
		}
		return RootPath;
	}
	return FolderPath;
}

void AActor::SetFolderPath(const FName& InNewFolderPath)
{
	if (IsUsingActorFolders(this))
	{
		UActorFolder* ActorFolder = nullptr;
		UWorld* World = GetWorld();
		if (!InNewFolderPath.IsNone() && World)
		{
			FFolder NewFolder(GetFolderRootObject(), InNewFolderPath);
			ActorFolder = FWorldPersistentFolders::GetActorFolder(NewFolder, World);
			if (!ActorFolder)
			{
				ActorFolder = FWorldPersistentFolders::GetActorFolder(NewFolder, World, /*bAllowCreate*/ true);
			}
		}
		SetFolderGuidInternal(ActorFolder ? ActorFolder->GetGuid() : FGuid());
	}
	else
	{
		SetFolderPathInternal(InNewFolderPath);
	}
}

void AActor::SetFolderGuidInternal(const FGuid& InFolderGuid, bool bInBroadcastChange)
{
	if ((FolderGuid == InFolderGuid) && FolderPath.IsNone())
	{
		return;
	}

	FName OldPath = !FolderPath.IsNone() ? FolderPath : GetFolderPath();
	
	Modify();
	FolderPath = NAME_None;
	FolderGuid = InFolderGuid;

	if (GEngine && bInBroadcastChange)
	{
		GEngine->BroadcastLevelActorFolderChanged(this, OldPath);
	}
}

void AActor::SetFolderPathInternal(const FName& InNewFolderPath, bool bInBroadcastChange)
{
	FName OldPath = FolderPath;
	if (InNewFolderPath.IsEqual(OldPath, ENameCase::CaseSensitive) && !FolderGuid.IsValid())
	{
		return;
	}

	Modify();
	FolderPath = InNewFolderPath;
	FolderGuid.Invalidate();

	if (GEngine && bInBroadcastChange)
	{
		GEngine->BroadcastLevelActorFolderChanged(this, OldPath);
	}
}

void AActor::SetFolderPath_Recursively(const FName& NewFolderPath)
{
	FActorEditorUtils::TraverseActorTree_ParentFirst(this, [&](AActor* InActor){
		InActor->SetFolderPath(NewFolderPath);
		return true;
	});
}

// Transfers some properties from the old actor
// Ideally, this should be revisited to implement something more generic.
void AActor::EditorReplacedActor(AActor* OldActor)
{
	// Don't update Content Bundle if new actor is using External Data Layer
	if (!ExternalDataLayerAsset)
	{
		// We can't rely on OldActor to transfer the ContentBundleGuid as it could have moved to a different Content Bundle
		// Resolve ContentBundleGuid from ActorPackage
		ContentBundleGuid = ContentBundlePaths::GetContentBundleGuidFromExternalActorPackagePath(GetPackage()->GetFName().ToString());
	}

	SetActorLabel(OldActor->GetActorLabel());
	Tags = OldActor->Tags;
	
	SetFolderPath(OldActor->GetFolderPath());
	
	if (CanChangeIsSpatiallyLoadedFlag())
	{
		SetIsSpatiallyLoaded(OldActor->bIsSpatiallyLoaded);
	}

	SetRuntimeGrid(OldActor->RuntimeGrid);

	const bool bUseLevelContext = true;
	const bool bIncludeParentDataLayers = false;
	TArray<const UDataLayerInstance*> ConstDataLayerInstances = OldActor->GetDataLayerInstancesInternal(bUseLevelContext, bIncludeParentDataLayers);
	if (!ConstDataLayerInstances.IsEmpty())
	{
		// Transfer old data layers (except the external data layer as it is handled at actor spawning)
		TArray<UDataLayerInstance*> DataLayerInstances;
		Algo::TransformIf(ConstDataLayerInstances, DataLayerInstances, [](const UDataLayerInstance* ConstDataLayerInstance) { return !ConstDataLayerInstance->IsA<UExternalDataLayerInstance>(); }, [](const UDataLayerInstance* ConstDataLayerInstance) { return const_cast<UDataLayerInstance*>(ConstDataLayerInstance); });
		IDataLayerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IDataLayerEditorModule>("DataLayerEditor");
		EditorModule.AddActorToDataLayers(this, DataLayerInstances);
	}

	// Broadcast that actor was replaced
	FEditorDelegates::OnEditorActorReplaced.Broadcast(OldActor, this);
}

void AActor::CheckForDeprecated()
{
	if ( GetClass()->HasAnyClassFlags(CLASS_Deprecated) )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_ActorIsObselete_Deprecated", "{ActorName} : Obsolete and must be removed! (Class is deprecated)" ), Arguments) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::ActorIsObselete));
	}
	// don't check to see if this is an abstract class if this is the CDO
	if ( !(GetFlags() & RF_ClassDefaultObject) && GetClass()->HasAnyClassFlags(CLASS_Abstract) )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_ActorIsObselete_Abstract", "{ActorName} : Obsolete and must be removed! (Class is abstract)" ), Arguments) ) )
			->AddToken(FMapErrorToken::Create(FMapErrors::ActorIsObselete));
	}
}

void AActor::CheckForErrors()
{
	int32 OldNumWarnings = FMessageLog("MapCheck").NumMessages(EMessageSeverity::Warning);
	CheckForDeprecated();
	if (OldNumWarnings < FMessageLog("MapCheck").NumMessages(EMessageSeverity::Warning))
	{
		return;
	}

	UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(RootComponent);
	if (PrimComp && (PrimComp->Mobility != EComponentMobility::Movable) && PrimComp->BodyInstance.bSimulatePhysics)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_StaticPhysNone", "{ActorName} : Static object with bSimulatePhysics set to true" ), Arguments) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::StaticPhysNone));
	}

	if (RootComponent)
	{
		const FVector LocalRelativeScale3D = RootComponent->GetRelativeScale3D();
		if (FMath::IsNearlyZero(LocalRelativeScale3D.X * LocalRelativeScale3D.Y * LocalRelativeScale3D.Z))
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
			FMessageLog("MapCheck").Error()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_InvalidDrawscale", "{ActorName} : Invalid DrawScale/DrawScale3D"), Arguments)))
				->AddToken(FMapErrorToken::Create(FMapErrors::InvalidDrawscale));
		}
	}

	// Route error checking to components.
	for (UActorComponent* ActorComponent : GetComponents())
	{
		if (ActorComponent && ActorComponent->IsRegistered())
		{
			ActorComponent->CheckForErrors();
		}
	}
}

bool AActor::GetReferencedContentObjects( TArray<UObject*>& Objects ) const
{
	if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(GetClass()))
	{
		Objects.AddUnique(Blueprint);
	}
	else if (UBlueprintGeneratedClass* BlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(GetClass()))
	{
		Objects.AddUnique(BlueprintGeneratedClass);
	}
	return true;
}

bool AActor::GetSoftReferencedContentObjects(TArray<FSoftObjectPath>& SoftObjects) const
{
	return false;
}

EDataValidationResult AActor::IsDataValid(FDataValidationContext& Context) const
{
	bool bSuccess = CheckDefaultSubobjects();
	if (!bSuccess)
	{
		FText ErrorMsg = FText::Format(LOCTEXT("IsDataValid_Failed_CheckDefaultSubobjectsInternal", "{0} failed CheckDefaultSubobjectsInternal()"), FText::FromString(GetName()));
		Context.AddError(ErrorMsg);
	}

	EDataValidationResult Result = bSuccess ? EDataValidationResult::Valid : EDataValidationResult::Invalid;

	// check the components
	for (const UActorComponent* Component : GetComponents())
	{
		if (Component)
		{
			// if any component is invalid, our result is invalid
			// in the future we may want to update this to say that the actor was not validated if any of its components returns EDataValidationResult::NotValidated
			EDataValidationResult ComponentResult = Component->IsDataValid(Context);
			if (ComponentResult == EDataValidationResult::Invalid)
			{
				Result = EDataValidationResult::Invalid;
			}
		}
	}

	return Result;
}

ELevelInstanceType AActor::GetLevelInstanceType() const
{
	// ChildActorComponent Actors forward the call their ParentActor 
	if (AActor* ParentActor = GetParentActor())
	{
		return ParentActor->GetLevelInstanceType();
	}

	return LevelInstanceType;
}

ELevelInstanceFlags AActor::GetLevelInstanceFlags() const
{
	// ChildActorComponent Actors forward the call their ParentActor 
	if (AActor* ParentActor = GetParentActor())
	{
		return ParentActor->GetLevelInstanceFlags();
	}

	return LevelInstanceFlags;
}

//---------------------------------------------------------------------------
// DataLayers (begin)

bool AActor::AddDataLayer(const UDataLayerInstance* DataLayerInstance)
{
	return DataLayerInstance->AddActor(this);
}

bool AActor::RemoveDataLayer(const UDataLayerInstance* DataLayerInstance)
{
	return DataLayerInstance->RemoveActor(this);
}

TArray<const UDataLayerInstance*> AActor::RemoveAllDataLayers()
{
	TArray<const UDataLayerInstance*> RemovedDataLayerInstances = GetDataLayerInstances();
	if (!RemovedDataLayerInstances.IsEmpty())
	{
		Modify();
		RemovedDataLayerInstances.SetNum(Algo::RemoveIf(
			RemovedDataLayerInstances, [this] (const UDataLayerInstance* DataLayerInstance) { return !DataLayerInstance->RemoveActor(this); }));
	}

	return RemovedDataLayerInstances;
}

TArray<const UDataLayerAsset*> AActor::ResolveDataLayerAssets(const TArray<TSoftObjectPtr<UDataLayerAsset>>& InDataLayerAssets) const
{
	TArray<const UDataLayerAsset*> ResolvedAssets;
	ResolvedAssets.Reserve(InDataLayerAssets.Num());
	Algo::TransformIf(InDataLayerAssets, ResolvedAssets, [](const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset) { return DataLayerAsset.IsValid(); }, [](const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset) { return DataLayerAsset.Get(); });
	return ResolvedAssets;
}

void AActor::ForceLoadDataLayerAssets() const
{
	for (const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset : DataLayerAssets)
	{
		DataLayerAsset.LoadSynchronous();
	}
}

TArray<const UDataLayerAsset*> AActor::GetDataLayerAssets(bool bIncludeExternalDataLayerAsset) const
{
	TArray<const UDataLayerAsset*> ResolveDataLayerAsset = ResolveDataLayerAssets(DataLayerAssets);
	if (bIncludeExternalDataLayerAsset && ExternalDataLayerAsset)
	{
		ResolveDataLayerAsset.Add(ExternalDataLayerAsset);
	}
	return ResolveDataLayerAsset;
}

TArray<FName> AActor::GetDataLayerInstanceNames() const
{
	TArray<FName> DataLayerInstanceNames;
	TArray<const UDataLayerInstance*> DataLayerInstances = GetDataLayerInstances();
	DataLayerInstanceNames.Reserve(DataLayerInstances.Num());
	for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
	{
		DataLayerInstanceNames.Add(DataLayerInstance->GetFName());
	}
	return DataLayerInstanceNames;
}

bool AActor::HasExternalContent() const
{
	check(!ExternalDataLayerAsset || ExternalDataLayerAsset->GetUID().IsValid());
	return ExternalDataLayerAsset ? true : GetContentBundleGuid().IsValid();
}

TArray<const UDataLayerInstance*> AActor::GetDataLayerInstancesForLevel() const
{
	const bool bUseLevelContext = true;
	return GetDataLayerInstancesInternal(bUseLevelContext);
}

void AActor::FixupDataLayers(bool bRevertChangesOnLockedDataLayer /*= false*/)
{
	// Always call here because this gets called in AActor::Presave and makes sure the SoftObjectPtrs are resolved before saving.
	TArray<const UDataLayerAsset*> ResolvedAssets = ResolveDataLayerAssets(DataLayerAssets);

	if (IsTemplate())
	{
		// Fixup old data: Template actors should never have an ExternalDataLayerAsset
		ExternalDataLayerAsset = nullptr;
		return;
	}

	if (!SupportsDataLayerType(UDataLayerInstance::StaticClass()))
	{
		DataLayerAssets.Empty();
	}

	if (ExternalDataLayerAsset && !SupportsDataLayerType(UExternalDataLayerInstance::StaticClass()))
	{
		ExternalDataLayerAsset = nullptr;
	}

	if (DataLayerAssets.IsEmpty())
	{
		return;
	}

	// Don't fixup if the actor is not part of a level (template actors, etc.)
	if (!GetLevel())
	{
		return;
	}

	// Don't fixup in game world
	UWorld* World = GetWorld();
	if ((GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor)) ||
		(World && (World->IsGameWorld() || (World->WorldType == EWorldType::Inactive))))
	{ 
		return;
	}

	// Remove any data layer with a Root external data layer different from the actor's external data layer
	// This case can happen after duplicating an actor using the actor editor context with a different external data layer
	TSet<TSoftObjectPtr<const UDataLayerAsset>> InvalidDataLayerAssets;
	if (!DataLayerAssets.IsEmpty())
	{
		for (const UDataLayerInstance* DataLayerInstance : GetDataLayerInstances())
		{
			if (const UExternalDataLayerInstance* RootExternalDataLayerInstance = DataLayerInstance->GetRootExternalDataLayerInstance(); RootExternalDataLayerInstance && (RootExternalDataLayerInstance->GetAsset() != ExternalDataLayerAsset))
			{
				InvalidDataLayerAssets.Add(DataLayerInstance->GetAsset());
			}
		}
	}
	
	// Cleanup Data Layer assets we can't reference (Private DLs)
	DataLayerAssets.SetNum(Algo::RemoveIf(DataLayerAssets, [this, &InvalidDataLayerAssets](const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset)
	{
		return InvalidDataLayerAssets.Contains(DataLayerAsset) ||
			!UDataLayerAsset::CanBeReferencedByActor(DataLayerAsset, this) ||
			// Make sure there are no EDLs in DataLayerAssets
			(DataLayerAsset.IsValid() && DataLayerAsset->IsA<UExternalDataLayerAsset>());
	}));

	// Remove duplicates
	TSet<TSoftObjectPtr<UDataLayerAsset>> UniqueDataLayerAssets(DataLayerAssets);
	DataLayerAssets = UniqueDataLayerAssets.Array();
	
	// Use Actor's DataLayerManager since the fixup is relative to this level
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(this);
	if (!DataLayerManager || !DataLayerManager->CanResolveDataLayers())
	{
		return;
	}
	
	if (bRevertChangesOnLockedDataLayer)
	{
		TArray<const UDataLayerAsset*> ResolvedPreEditChangeAssets = ResolveDataLayerAssets(PreEditChangeDataLayers);
		// Since it's not possible to prevent changes of particular elements of an array, rollback change on read-only DataLayers.
		TSet<const UDataLayerAsset*> PreEdit(ResolvedPreEditChangeAssets);
		TSet<const UDataLayerAsset*> PostEdit(ResolvedAssets);

		auto DifferenceContainsLockedDataLayers = [DataLayerManager](const TSet<const UDataLayerAsset*>& A, const TSet<const UDataLayerAsset*>& B)
		{
			TSet<const UDataLayerAsset*> Diff = A.Difference(B);
			for (const UDataLayerAsset* DataLayerAsset : Diff)
			{
				const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerAsset);
				if (DataLayerInstance && DataLayerInstance->IsReadOnly())
				{
					return true;
				}
			}
			return false;
		};
					
		if (DifferenceContainsLockedDataLayers(PreEdit, PostEdit) || 
			DifferenceContainsLockedDataLayers(PostEdit, PreEdit))
		{
			DataLayerAssets = PreEditChangeDataLayers;
		}
	}

	auto CleanupDataLayers = [this, DataLayerManager](auto& DataLayerArray)
	{
		using ArrayType = typename TRemoveReference<decltype(DataLayerArray)>::Type;
					
		ArrayType ExistingDataLayer;
		for (int32 Index = 0; Index < DataLayerArray.Num();)
		{
			auto& DataLayer = DataLayerArray[Index];
			if (!DataLayerManager->GetDataLayerInstance(DataLayer) || ExistingDataLayer.Contains(DataLayer))
			{
				DataLayerArray.RemoveAtSwap(Index);
			}
			else
			{
				ExistingDataLayer.Add(DataLayer);
				++Index;
			}
		}
	};

	// Only invalidate DataLayerAssets on cook. In Editor we want to be able to re-resolve if the asset gets readded to the WorldDataLayers actor
	if (IsRunningCookCommandlet())
	{
		// Get a new resolved array in case DataLayerAssets changed
		TArray<const UDataLayerAsset*> CleanedUpDataLayers = ResolveDataLayerAssets(DataLayerAssets);
		CleanupDataLayers(CleanedUpDataLayers);
		DataLayerAssets.Empty(CleanedUpDataLayers.Num());
		Algo::Transform(CleanedUpDataLayers, DataLayerAssets, [](const UDataLayerAsset* DataLayerAsset) { return TSoftObjectPtr<UDataLayerAsset>(const_cast<UDataLayerAsset*>(DataLayerAsset)); });
	}
}

bool AActor::IsPropertyChangedAffectingDataLayers(FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		static const FName NAME_DataLayerAssets = GET_MEMBER_NAME_CHECKED(AActor, DataLayerAssets);
		static const FName NAME_ExternalDataLayerAsset = GET_MEMBER_NAME_CHECKED(AActor, ExternalDataLayerAsset);

		const FName PropertyName = PropertyChangedEvent.GetPropertyName();
		if (PropertyName == NAME_ExternalDataLayerAsset ||
			(PropertyName == NAME_DataLayerAssets &&
			 ((PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet) ||
			 (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear) ||
			 (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)))
			)
		{
			return true;
		}
	}
	return false;
}

static bool HasComponentForceActorNoDataLayers(const AActor* InActor)
{
	bool bHasComponentForceActorNoDataLayers = false;
	InActor->ForEachComponent(false, [&bHasComponentForceActorNoDataLayers](const UActorComponent* Component)
	{
		bHasComponentForceActorNoDataLayers |= Component->ForceActorNoDataLayers();
	});
	return bHasComponentForceActorNoDataLayers;
}

bool AActor::SupportsDataLayerType(TSubclassOf<UDataLayerInstance> InDataLayerType) const
{
	ULevel* Level = GetLevel();
	const bool bIsLevelIsPartitioned = Level ? Level->bIsPartitioned || Level->IsWorldPartitionRuntimeCell() : true;
	const bool bHasComponentForceActorNoDataLayers = HasComponentForceActorNoDataLayers(this);
	const bool bActorTypeSupportsDataLayerType = InDataLayerType->IsChildOf<UExternalDataLayerInstance>() ? ActorTypeSupportsExternalDataLayer() : ActorTypeSupportsDataLayer();
	
	return (bIsLevelIsPartitioned &&
		!bHasComponentForceActorNoDataLayers &&
		bActorTypeSupportsDataLayerType &&
		IsPackageExternal() &&
		!FActorEditorUtils::IsABuilderBrush(this) &&
		!GetClass()->GetDefaultObject<AActor>()->bHiddenEd);
}

bool AActor::CanAddDataLayer(const UDataLayerInstance* InDataLayerInstance, FText* OutReason) const
{
	auto PassesAssetReferenceFiltering = [](const UObject * InReferencingObject, const UDataLayerAsset * InDataLayerAsset, FText* OutReason)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAsset(FAssetData(InReferencingObject));
		TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
		return AssetReferenceFilter.IsValid() ? AssetReferenceFilter->PassesFilter(FAssetData(InDataLayerAsset), OutReason) : true;
	};

	if (!InDataLayerInstance)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddDataLayerInvalidDataLayerInstance", "Invalid data layer instance.");
		}
		return false;
	}

	if (!SupportsDataLayerType(InDataLayerInstance->GetClass()))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddDataLayerActorDoesntSupportDataLayerType", "Actor doesn't support this data layer type.");
		}
		return false;
	}

	if (const UDataLayerAsset* DataLayerAsset = InDataLayerInstance->GetAsset())
	{
		if (!PassesAssetReferenceFiltering(this, DataLayerAsset, OutReason))
		{
			return false;
		}

		if (const UExternalDataLayerAsset* InExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(DataLayerAsset))
		{
			if (ContentBundleGuid.IsValid())
			{
				if (OutReason)
				{
					*OutReason = LOCTEXT("CantAddDataLayerActorAlreadyAssignedToContentBundle", "Actor is already assigned to a content bundle.");
				}
				return false;
			}
			else if (ExternalDataLayerAsset)
			{
				if (OutReason)
				{
					if (ExternalDataLayerAsset == DataLayerAsset)
					{
						*OutReason = LOCTEXT("CantAddDataLayerActorAlreadyAssignedToExternalDataLayer", "Actor is already assigned to this external data layer.");
					}
					else
					{
						*OutReason = LOCTEXT("CantAddDataLayerActorAlreadyAssignedToAnotherExternalDataLayer", "Actor is already assigned to another external data layer.");
					}
				}
				return false;
			}
		}
		else
		{
			if (DataLayerAssets.Contains(DataLayerAsset))
			{
				if (OutReason)
				{
					*OutReason = LOCTEXT("CantAddDataLayerActorAlreadyAssignedToDataLayer", "Actor is already assigned to this data layer.");
				}
				return false;
			}
		}
	}
	else
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddDataLayerInvalidDataLayerAsset", "Invalid data layer asset.");
		}
		return false;
	}

	return true;
}

bool FAssignActorDataLayer::AddDataLayerAsset(AActor* InActor, const UDataLayerAsset* InDataLayerAsset)
{
	check(InDataLayerAsset != nullptr);

	if (const UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(InDataLayerAsset))
	{
		if (!InActor->ExternalDataLayerAsset)
		{
			InActor->Modify();
			InActor->ExternalDataLayerAsset = ExternalDataLayerAsset;
			return true;
		}

		UE_CLOGF(InActor->ExternalDataLayerAsset != ExternalDataLayerAsset, LogActor, Warning, "Trying to assign external data layer %ls on actor %ls while %ls is already assigned.", 
			*ExternalDataLayerAsset->GetPathName(), *InActor->GetActorNameOrLabel(), *InActor->ExternalDataLayerAsset->GetPathName());
	}
	else if (!InActor->DataLayerAssets.Contains(InDataLayerAsset))
	{
		InActor->Modify();
		InActor->DataLayerAssets.Add(const_cast<UDataLayerAsset*>(InDataLayerAsset));
		return true;
	}

	return false;
}

bool FAssignActorDataLayer::RemoveDataLayerAsset(AActor* InActor, const UDataLayerAsset* InDataLayerAsset)
{
	check(InDataLayerAsset != nullptr);
	
	if (InActor->ExternalDataLayerAsset == InDataLayerAsset)
	{
		InActor->Modify();
		InActor->ExternalDataLayerAsset = nullptr;
		return true;
	}
	else if (InActor->DataLayerAssets.Contains(InDataLayerAsset))
	{
		InActor->Modify();
		InActor->DataLayerAssets.Remove(const_cast<UDataLayerAsset*>(InDataLayerAsset));
		return true;
	}

	return false;
}

// DataLayers (end)
//---------------------------------------------------------------------------

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
