// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ActorEditorContext/ScopedActorEditorContextFromActor.h"
#include "Subsystems/ActorEditorContextSubsystem.h"

FScopedActorEditorContextFromActor::FScopedActorEditorContextFromActor(AActor* InActor)
{
	UActorEditorContextSubsystem::Get()->PushContext();
	UActorEditorContextSubsystem::Get()->InitializeContextFromActor(InActor);
}

FScopedActorEditorContextFromActor::~FScopedActorEditorContextFromActor()
{
	UActorEditorContextSubsystem::Get()->PopContext();
}

#endif