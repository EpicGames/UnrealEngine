// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ActorEditorContext/ScopedActorEditorContextSetExternalDataLayerAsset.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "WorldPartition/DataLayer/IDataLayerEditorModule.h"
#include "Modules/ModuleManager.h"

FScopedActorEditorContextSetExternalDataLayerAsset::FScopedActorEditorContextSetExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	const bool bDuplicateContext = true;
	UActorEditorContextSubsystem::Get()->PushContext(bDuplicateContext);
	IDataLayerEditorModule& DataLayerEditorModule = FModuleManager::LoadModuleChecked<IDataLayerEditorModule>("DataLayerEditor");
	DataLayerEditorModule.SetActorEditorContextCurrentExternalDataLayer(InExternalDataLayerAsset);
}

FScopedActorEditorContextSetExternalDataLayerAsset::~FScopedActorEditorContextSetExternalDataLayerAsset()
{
	UActorEditorContextSubsystem::Get()->PopContext();
}

#endif