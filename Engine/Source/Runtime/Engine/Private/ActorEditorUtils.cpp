// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorEditorUtils.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Engine/Brush.h"
#include "LevelUtils.h"

#if WITH_EDITOR
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"
#include "Kismet2/ComponentEditorUtils.h"
#endif


#define LOCTEXT_NAMESPACE "ActorEditorUtils"

namespace FActorEditorUtils
{
	bool IsABuilderBrush( const AActor* InActor )
	{
		bool bIsBuilder = false;
#if WITH_EDITOR		
		if ( InActor && InActor->GetWorld() && !InActor->HasAnyFlags(RF_ClassDefaultObject) )
		{
			ULevel* ActorLevel = InActor->GetLevel();
			if ((ActorLevel != nullptr) && (ActorLevel->Actors.Num() >= 2))
			{
				// If the builder brush exists then it will be the 2nd actor in the actors array.
				ABrush* BuilderBrush = Cast<ABrush>(ActorLevel->Actors[1]);
				// If the second actor is not a brush then it certainly cannot be the builder brush.
				if ((BuilderBrush != nullptr) && (BuilderBrush->GetBrushComponent() != nullptr) && (BuilderBrush->Brush != nullptr))
				{
					bIsBuilder = (BuilderBrush == InActor);
				}
			}			
		}
#endif
		return bIsBuilder;
	}

	bool IsABrush( const AActor* InActor )
	{
		return Cast<ABrush>(InActor) != nullptr;
	}

	bool IsAPreviewOrInactiveActor( const AActor* InActor )
	{
		UWorld* World = InActor ? InActor->GetWorld() : NULL;
		return World && (World->WorldType == EWorldType::EditorPreview || World->WorldType == EWorldType::Inactive);
	}

	void GetEditableComponents( const AActor* InActor, TArray<UActorComponent*>& OutEditableComponents )
	{
		for (UActorComponent* Component : InActor->GetComponents())
		{
			if (Component)
			{
#if WITH_EDITOR
				if (Component->CreationMethod == EComponentCreationMethod::Native)
				{
					// Make sure it's an exposed native component
					if (FComponentEditorUtils::GetPropertyForEditableNativeComponent(Component))
					{
						OutEditableComponents.Add(Component);
					}
				}
				else if (Component->CreationMethod == EComponentCreationMethod::Instance)
#else
				if (!Component->IsCreatedByConstructionScript())
#endif
				{
					OutEditableComponents.Add(Component);
				}
			}
		}
	}

	bool TraverseActorTree_ParentFirst(AActor* InActor, TFunctionRef<bool(AActor*)> InPredicate, bool bIncludeThisActor)
	{
		if (!InActor)
		{
			return true;
		}

		if (bIncludeThisActor && !InPredicate(InActor))
		{
			return false;
		}

		if( InActor->GetRootComponent() )
		{
			for(USceneComponent* ChildComponent : InActor->GetRootComponent()->GetAttachChildren())
			{
				AActor* ChildActor = ChildComponent ? ChildComponent->GetOwner() : nullptr;
				if(ChildActor && ChildActor != InActor && !TraverseActorTree_ParentFirst(ChildActor, InPredicate))
				{
					return false;
				}
			}
		}
		
		return true;
	}

	bool TraverseActorTree_ChildFirst(AActor* InActor, TFunctionRef<bool(AActor*)> InPredicate, bool bIncludeThisActor)
	{
		if (!InActor)
		{
			return true;
		}

		for (USceneComponent* ChildComponent : InActor->GetRootComponent()->GetAttachChildren())
		{
			AActor* ChildActor = ChildComponent ? ChildComponent->GetOwner() : nullptr;
			if (ChildActor && ChildActor != InActor && !TraverseActorTree_ChildFirst(ChildActor, InPredicate))
			{
				return false;
			}
		}

		return !bIncludeThisActor || InPredicate(InActor);
	}

	bool ValidateActorName(const FText& InName, FText& OutErrorMessage)
	{
		return ValidateActorLabel(FStringView(InName.ToString()), OutErrorMessage);
	}

	bool ValidateActorLabel(FStringView InName, FText& OutErrorMessage)
	{
		InName.TrimStartAndEndInline();

		if (InName.Len() == 0)
		{
			OutErrorMessage = LOCTEXT("ValidateActorNameString_Empty", "Actor names cannot be empty");
			return false;
		}

		if (InName.Len() >= NAME_SIZE)
		{
			OutErrorMessage = FText::Format(LOCTEXT("ValidateActorNameString_TooLong", "Names must be less than {0} characters long."), NAME_SIZE);
			return false;
		}

		if (InName.Compare("None", ESearchCase::IgnoreCase) == 0)
		{
			OutErrorMessage = LOCTEXT("ValidateActorNameString_ReservedNameNone", "\"None\" is a reserved term and cannot be used for actor names");
			return false;
		}

		return true;
	}
#if WITH_EDITOR
	bool IsActorDisplayable(const AActor* Actor)
	{
		return Actor &&
			   IsValid(Actor) &&										// We don't want to show actors that are about to go away
			   Actor->IsEditable() &&									// Only show actors that are allowed to be selected and drawn in editor
			   Actor->IsListedInSceneOutliner() &&
			   !Actor->IsTemplate() &&									// Should never happen, but we never want CDOs displayed
			   !IsABuilderBrush(Actor) &&								// Don't show the builder brush
			   !Actor->IsA(AWorldSettings::StaticClass()) &&	// Don't show the WorldSettings actor, even though it is technically editable
			   FLevelUtils::IsLevelVisible(Actor->GetLevel());			// Only show Actors whose level is visible
	}
#endif // WITH_EDITOR

}

#undef LOCTEXT_NAMESPACE


