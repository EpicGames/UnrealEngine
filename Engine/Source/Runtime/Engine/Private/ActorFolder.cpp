// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 * UActorFolder implementation
 *=============================================================================*/

#include "ActorFolder.h"
#include "Folder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFolder)

#if WITH_EDITOR

#include "Engine/Level.h"
#include "ExternalPackageHelper.h"
#include "ActorFolderDesc.h"
#include "UObject/AssetRegistryTagsContext.h"

UActorFolder* UActorFolder::Create(ULevel* InLevel, const FString& InFolderLabel, UActorFolder* InParent)
{
	const FGuid NewFolderGuid = FGuid::NewGuid();

	// We generate a globally unique name to avoid any potential clash of 2 users creating the same folder
	FString FolderShortName = UActorFolder::StaticClass()->GetName() + TEXT("_UID_") + NewFolderGuid.ToString(EGuidFormats::UniqueObjectGuid);
	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> GloballyUniqueObjectPath;
	GloballyUniqueObjectPath += InLevel->GetPathName();
	GloballyUniqueObjectPath += TEXT(".");
	GloballyUniqueObjectPath += FolderShortName;

	const bool bIsTransientFolder = (InLevel->IsInstancedLevel() && !InLevel->IsPersistentLevel()) || InLevel->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor);
	const bool bUseExternalObject = InLevel->IsUsingExternalObjects() && !bIsTransientFolder;
	const bool bShouldDirtyLevel = !bUseExternalObject;
	const EObjectFlags Flags = (bIsTransientFolder ? RF_Transient : RF_NoFlags) | RF_Transactional;

	UPackage* ExternalPackage = bUseExternalObject ? FExternalPackageHelper::CreateExternalPackage(InLevel, *GloballyUniqueObjectPath) : nullptr;

	UActorFolder* ActorFolder = NewObject<UActorFolder>(InLevel, UActorFolder::StaticClass(), FName(FolderShortName), Flags, nullptr, /*bCopyTransientsFromClassDefaults*/false, /*InstanceGraph*/nullptr, ExternalPackage);
	check(ActorFolder);
	ActorFolder->FolderGuid = NewFolderGuid;
	ActorFolder->SetLabel(InFolderLabel);
	ActorFolder->SetParent(InParent);
	ActorFolder->SetIsInitiallyExpanded(true);

	FLevelActorFoldersHelper::AddActorFolder(InLevel, ActorFolder, bShouldDirtyLevel);
	return ActorFolder;
}

bool UActorFolder::IsAsset() const
{
	// Actor Folders are considered assets to allow using the asset logic for save dialogs, etc.
	// Also, they return true even if pending kill, in order to show up as deleted in these dialogs.
	return IsPackageExternal() && !GetPackage()->HasAnyFlags(RF_Transient) && !HasAnyFlags(RF_Transient | RF_ClassDefaultObject) && !GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor);
}

void UActorFolder::PostLoad()
{
	Super::PostLoad();

	FolderLabel.TrimStartAndEndInline();
}

namespace ActorFolder
{
	static const FName NAME_FolderGuid("FolderGuid");
	static const FName NAME_ParentFolderGuid("ParentFolderGuid");
	static const FName NAME_FolderLabel("FolderLabel");
	static const FName NAME_FolderInitiallyExpanded("FolderInitiallyExpanded");
	static const FName NAME_FolderIsDeleted("FolderIsDeleted");
	static const FName NAME_OuterPackageName("OuterPackageName");
};

void UActorFolder::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	Context.AddTag(FAssetRegistryTag(ActorFolder::NAME_FolderGuid, *FolderGuid.ToString(), FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(ActorFolder::NAME_ParentFolderGuid, *ParentFolderGuid.ToString(), FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(ActorFolder::NAME_FolderLabel, *FolderLabel, FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(ActorFolder::NAME_FolderInitiallyExpanded, bFolderInitiallyExpanded ? TEXT("1") : TEXT("0"), FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(ActorFolder::NAME_FolderIsDeleted, bIsDeleted ? TEXT("1") : TEXT("0"), FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(ActorFolder::NAME_OuterPackageName, HasAnyFlags(RF_ClassDefaultObject) ? TEXT("") : *GetOuterULevel()->GetPackage()->GetName(), FAssetRegistryTag::TT_Hidden));
}

FActorFolderDesc UActorFolder::GetAssetRegistryInfoFromPackage(FName ActorFolderPackageName)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	FActorFolderDesc ActorFolderDesc;
	TArray<FAssetData> Assets;
	AssetRegistry.GetAssetsByPackageName(ActorFolderPackageName, Assets, true);
	check(Assets.Num() <= 1);
	if (Assets.Num() == 1)
	{
		const FAssetData& Asset = Assets[0];
		static const FTopLevelAssetPath NAME_ActorFolder(TEXT("/Script/Engine"), TEXT("ActorFolder"));
		check(Asset.AssetClassPath == NAME_ActorFolder);
		{
			FString Value;
			if (Asset.GetTagValue(ActorFolder::NAME_FolderGuid, Value))
			{
				FGuid::Parse(Value, ActorFolderDesc.FolderGuid);
			}
			if (Asset.GetTagValue(ActorFolder::NAME_ParentFolderGuid, Value))
			{
				FGuid::Parse(Value, ActorFolderDesc.ParentFolderGuid);
			}
			if (Asset.GetTagValue(ActorFolder::NAME_FolderLabel, Value))
			{
				ActorFolderDesc.FolderLabel = Value;
			}
			if (Asset.GetTagValue(ActorFolder::NAME_FolderInitiallyExpanded, Value))
			{
				ActorFolderDesc.bFolderInitiallyExpanded = (Value == TEXT("1"));
			}
			if (Asset.GetTagValue(ActorFolder::NAME_FolderIsDeleted, Value))
			{
				ActorFolderDesc.bFolderIsDeleted = (Value == TEXT("1"));
			}
			if (Asset.GetTagValue(ActorFolder::NAME_OuterPackageName, Value))
			{
				ActorFolderDesc.OuterPackageName = Value;
			}
		}
	}
	return ActorFolderDesc;
}

void UActorFolder::SetLabel(const FString& InFolderLabel)
{
	FString TrimmedFolderLabel = InFolderLabel.TrimStartAndEnd();
	check(IsValid());
	if (!FolderLabel.Equals(TrimmedFolderLabel, ESearchCase::CaseSensitive))
	{
		Modify();
		FString OldFolderLabel = FolderLabel;
		FolderLabel = TrimmedFolderLabel;
		GetTypedOuter<ULevel>()->OnFolderLabelChanged(this, OldFolderLabel);
	}
}

void UActorFolder::SetIsInitiallyExpanded(bool bInFolderInitiallyExpanded)
{
	check(IsValid());
	if (bFolderInitiallyExpanded != bInFolderInitiallyExpanded)
	{
		Modify();
		bFolderInitiallyExpanded = bInFolderInitiallyExpanded;
	}
}

void UActorFolder::SetParent(UActorFolder* InParent)
{
	if ((this != InParent) && (GetParent() != InParent))
	{
		Modify();
		ParentFolderGuid = InParent ? InParent->GetGuid() : FGuid();
	}
}

FString UActorFolder::GetDisplayName() const
{
	if (IsMarkedAsDeleted())
	{
		return FString::Printf(TEXT("<Deleted> %s"), *GetLabel());
	}
	return GetPath().ToString();
}

void UActorFolder::MarkAsDeleted()
{
	Modify();

	ULevel* Level = GetTypedOuter<ULevel>();
	
	auto HasParent = [](UActorFolder* InFolder, UActorFolder* InParent)
	{
		UActorFolder* Parent = InFolder->GetParent(/*bSkipDeleted*/false);
		while (Parent)
		{
			if (Parent == InParent)
			{
				return true;
			}
			Parent = Parent->GetParent(/*bSkipDeleted*/false);
		}
		return false;
	};

	TArray<UActorFolder*> FoldersToDelete;
	TMap<UActorFolder*, UActorFolder*> DuplicateFolders;
	const FName PathToDelete = GetPath();
	// Find all SubPaths of PathToDelete
	Level->ForEachActorFolder([this, Level, PathToDelete, &HasParent, &DuplicateFolders, &FoldersToDelete](UActorFolder* ActorFolder)
	{
		if (HasParent(ActorFolder, this))
		{
			// Get child folder new path if parent is deleted
			FName ChildFolderNewPath = ActorFolder->GetPathInternal(this);
			if (UActorFolder* ExistingFolder = Level->GetActorFolder(ChildFolderNewPath))
			{
				DuplicateFolders.Add(ActorFolder, ExistingFolder);
				FoldersToDelete.Add(ActorFolder);
			}
		}
		return true;
	}, /*bSkipDeleted*/ true);

	// Sort in descending order so children will be deleted before parents
	FoldersToDelete.Sort([](const UActorFolder& FolderA, const UActorFolder& FolderB)
	{
		return FolderB.GetPath().LexicalLess(FolderA.GetPath());
	});
	
	for (UActorFolder* FolderToDelete : FoldersToDelete)
	{
		UActorFolder* NewParent = DuplicateFolders.FindChecked(FolderToDelete);
		// First move duplicate folder under the single folder we keep. Use a unique name to avoid dealing with name clash
		const FFolder OldFolder = FolderToDelete->GetFolder();
		const FString NewPath = FString::Printf(TEXT("%s/%s_Duplicate_%s"), *NewParent->GetPath().ToString(), *FolderToDelete->GetLabel(), *FGuid::NewGuid().ToString());
		const FFolder NewFolder = FFolder(OldFolder.GetRootObject(), FName(NewPath));
		FLevelActorFoldersHelper::RenameFolder(Level, OldFolder, NewFolder);
		// Then delete (mark as deleted) this folder
		FLevelActorFoldersHelper::DeleteFolder(Level, FolderToDelete->GetFolder());
	}

	// Deleting a folder must not modify actors part of it nor sub folders.
	// Here, we simply mark the folder as deleted. 
	// When marked as deleted, the folder will act as a redirector to its parent.
	check(!bIsDeleted);
	bIsDeleted = true;
	Level->OnFolderMarkAsDeleted(this);
}

UActorFolder* UActorFolder::GetParent(bool bSkipDeleted) const
{
	return ParentFolderGuid.IsValid() ? GetOuterULevel()->GetActorFolder(ParentFolderGuid, bSkipDeleted) : nullptr;
}

FName UActorFolder::GetPath() const
{
	return GetPathInternal(nullptr);
}

FORCEINLINE void BuildPath(TStringBuilder<512>& OutStringBuilder, const UActorFolder* InSkipFolder, const UActorFolder* InCurrentFolder)
{
	if (InCurrentFolder)
	{
		BuildPath(OutStringBuilder, InSkipFolder, InCurrentFolder->GetParent());
		if (InCurrentFolder != InSkipFolder)
		{
			if (OutStringBuilder.Len())
			{
				OutStringBuilder += TEXT("/");
			}
			OutStringBuilder += InCurrentFolder->GetLabel();
		}
	}
};

FName UActorFolder::GetPathInternal(UActorFolder* InSkipFolder) const
{
	TStringBuilder<512> StringBuilder;
	BuildPath(StringBuilder, InSkipFolder, GetParent());
	if (IsValid())
	{
		if (StringBuilder.Len())
		{
			StringBuilder += TEXT("/");
		}
		StringBuilder += FolderLabel;
	}
	return FName(*StringBuilder);
}

void UActorFolder::FixupParentFolder()
{
	if (ParentFolderGuid.IsValid() && !GetOuterULevel()->GetActorFolder(ParentFolderGuid, /*bSkipDeleted*/ false))
	{
		// Here we don't warn anymore since there's a supported workflow where we allow to delete actor folders 
		// even when referenced by other folders (i.e. when the end result remains the root)
		Modify();
		ParentFolderGuid.Invalidate();
	}
}

void UActorFolder::Fixup()
{
	if (!IsMarkedAsDeleted() && ParentFolderGuid.IsValid())
	{
		UActorFolder* Parent = GetParent();
		FGuid ValidParentFolderGuid = Parent ? Parent->GetGuid() : FGuid();
		if (ParentFolderGuid != ValidParentFolderGuid)
		{
			Modify();
			ParentFolderGuid = ValidParentFolderGuid;
		}
	}
}

FFolder UActorFolder::GetFolder() const
{
	// Resolve FFolder::FRootObject for this ActorFolder using its outer level
	ULevel* OuterLevel = GetOuterULevel();
	check(OuterLevel);
	check(FolderGuid.IsValid());
	FFolder::FRootObject RootObject = FFolder::GetOptionalFolderRootObject(OuterLevel).Get(FFolder::GetInvalidRootObject());
	
	// Detect case where returned root object is different from outer level (this is the case for UWorldPartitionLevelStreamingDynamic::GetFolderRootObject).
	ULevel* RootObjectAssociatedLevel = FFolder::GetRootObjectAssociatedLevel(RootObject);
	if (RootObjectAssociatedLevel != OuterLevel)
	{
		// Build and return a FFolder using the root object and ActorFolder's path
		return FFolder(RootObject, GetPath());
	}

	// Build and return a FFolder using the root object and ActorFolder's guid
	FFolder Folder = FFolder(RootObject, FolderGuid);
	check(Folder.GetActorFolder() == this);
	return Folder;
}

void UActorFolder::SetPackageExternal(bool bInExternal, bool bShouldDirty)
{
	FExternalPackageHelper::SetPackagingMode(this, GetOuterULevel(), bInExternal, bShouldDirty);
}

#endif
