// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolderDesc.h"
#include "ActorFolder.h"
#include "ExternalPackageHelper.h"

#if WITH_EDITOR

/** 
  * FActorFolderDesc Implementation
  */

FActorFolderDesc::FActorFolderDesc()
	: bFolderInitiallyExpanded(true)
	, bFolderIsDeleted(false)
{}

FString FActorFolderDesc::GetDisplayName() const
{
	if (IsFolderDeleted())
	{
		return FString::Printf(TEXT("<Deleted> %s"), *GetFolderLabel());
	}
	return GetPath();
}

FString FActorFolderDesc::GetPath() const
{
	FActorFolderDescsContext Context(*this);

	TStringBuilder<1024> StringBuilder;
	if (!IsFolderDeleted())
	{
		StringBuilder += GetFolderLabel();
	}

	const FActorFolderDesc* Parent = Context.GetParentActorFolderDesc(*this);
	while (Parent)
	{
		StringBuilder.Prepend(TEXT("/"));
		StringBuilder.Prepend(Parent->GetFolderLabel());
		Parent = Context.GetParentActorFolderDesc(*Parent);
	}
	return *StringBuilder;
}

/**
  * FActorFolderDescsContext Implementation
  */

FActorFolderDescsContext::FActorFolderDescsContext(const FActorFolderDesc& InActorFolderDesc)
{
	const FString ActorFoldersRoot = FExternalPackageHelper::GetExternalObjectsPath(InActorFolderDesc.GetOuterPackageName());
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.ScanPathsSynchronous({ ActorFoldersRoot }, /*bForceRescan*/false, /*bIgnoreDenyListScanFilters*/false);
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = true;
	Filter.ClassPaths.Add(UActorFolder::StaticClass()->GetClassPathName());
	Filter.PackagePaths.Add(*ActorFoldersRoot);
	TArray<FAssetData> Assets;
	AssetRegistry.GetAssets(Filter, Assets);

	for (auto Asset : Assets)
	{
		FActorFolderDesc ActorFolderDesc = UActorFolder::GetAssetRegistryInfoFromPackage(Asset.PackageName);
		if (ActorFolderDesc.GetFolderGuid().IsValid())
		{
			ActorFolders.Add(ActorFolderDesc.GetFolderGuid(), ActorFolderDesc);
		}
	}
}

const FActorFolderDesc* FActorFolderDescsContext::GetActorFolderDesc(const FGuid& InFolderGuid)
{
	return ActorFolders.Find(InFolderGuid);
}

const FActorFolderDesc* FActorFolderDescsContext::GetParentActorFolderDesc(const FActorFolderDesc& InActorFolderDesc)
{
	const FActorFolderDesc* Parent = GetActorFolderDesc(InActorFolderDesc.GetParentFolderGuid());
	if (Parent && Parent->IsFolderDeleted())
	{
		return GetParentActorFolderDesc(*Parent);
	}
	return Parent;
}

#endif
