// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPartition/ActorPartitionSubsystem.h"
#include "Math/IntRect.h"
#include "WorldPartition/ActorPartition/PartitionActorDesc.h"
#include "Subsystems/Subsystem.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/DataLayer/DataLayerEditorContext.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "EngineUtils.h"

#if WITH_EDITOR
#include "Algo/Transform.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"
#include "DeletedObjectPlaceholder.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorPartitionSubsystem)

DEFINE_LOG_CATEGORY_STATIC(LogActorPartitionSubsystem, All, All);

#if WITH_EDITOR

// @todo_ow: All this can be converted back to DataLayerEditorContextHash when EDL replaces ContentBundles
namespace FActorPartitionContextHash
{
	uint32 Get(const FGuid& InContentBundleGuid, uint32 InDataLayerEditorContextHash)
	{
		return InContentBundleGuid.IsValid() ? FCrc::TypeCrc32(InContentBundleGuid, InDataLayerEditorContextHash) : InDataLayerEditorContextHash;
	}

	uint32 Get(const FGuid& InContentBundleGuid, UWorld* InWorld, const TArray<FName>& InDataLayerInstanceNames)
	{
		FDataLayerEditorContext DataLayerEditorContext(InWorld, InDataLayerInstanceNames);
		return Get(InContentBundleGuid, DataLayerEditorContext.GetHash());
	}
};

FActorPartitionGetParams::FActorPartitionGetParams(const TSubclassOf<APartitionActor>& InActorClass, bool bInCreate, ULevel* InLevelHint, const FVector& InLocationHint, uint32 InGridSize, const FGuid& InGuidHint, bool bInBoundsSearch, TFunctionRef<void(APartitionActor*)> InActorCreated)
	: ActorClass(InActorClass)
	, bCreate(bInCreate)
	, LocationHint(InLocationHint)
	, LevelHint(InLevelHint)
	, GuidHint(InGuidHint)
	, GridSize(InGridSize)
	, bBoundsSearch(bInBoundsSearch)
	, ActorCreatedCallback(InActorCreated)
{
}

void FActorPartitionGridHelper::ForEachIntersectingCell(const TSubclassOf<APartitionActor>& InActorClass, const FBox& InBounds, ULevel* InLevel, TFunctionRef<bool(const UActorPartitionSubsystem::FCellCoord&, const FBox&)> InOperation, uint32 InGridSize)
{
	const uint32 GridSize = InGridSize > 0 ? InGridSize : InActorClass->GetDefaultObject<APartitionActor>()->GetDefaultGridSize(InLevel->GetWorld());
	const UActorPartitionSubsystem::FCellCoord MinCellCoords = UActorPartitionSubsystem::FCellCoord::GetCellCoord(InBounds.Min, InLevel, GridSize);
	const UActorPartitionSubsystem::FCellCoord MaxCellCoords = UActorPartitionSubsystem::FCellCoord::GetCellCoord(InBounds.Max, InLevel, GridSize);

	for (int32 z = MinCellCoords.Z; z <= MaxCellCoords.Z; z++)
	{
		for (int32 y = MinCellCoords.Y; y <= MaxCellCoords.Y; y++)
		{
			for (int32 x = MinCellCoords.X; x <= MaxCellCoords.X; x++)
			{
				UActorPartitionSubsystem::FCellCoord CellCoords(x, y, z, InLevel);
				const FVector Min = FVector(
					CellCoords.X * GridSize,
					CellCoords.Y * GridSize,
					CellCoords.Z * GridSize
				);
				const FVector Max = Min + FVector(GridSize);
				FBox CellBounds(Min, Max);

				if (!InOperation(MoveTemp(CellCoords), MoveTemp(CellBounds)))
				{
					return;
				}
			}
		}
	}
}

void FActorPartitionGridHelper::ForEachIntersectingCell(const TSubclassOf<APartitionActor>& InActorClass, const FIntRect& InRect, ULevel* InLevel, TFunctionRef<bool(const UActorPartitionSubsystem::FCellCoord&, const FIntRect&)> InOperation, uint32 InGridSize)
{
	const uint32 GridSize = InGridSize > 0 ? InGridSize : InActorClass->GetDefaultObject<APartitionActor>()->GetDefaultGridSize(InLevel->GetWorld());
	const UActorPartitionSubsystem::FCellCoord MinCellCoords = UActorPartitionSubsystem::FCellCoord::GetCellCoord(InRect.Min, InLevel, GridSize);
	const UActorPartitionSubsystem::FCellCoord MaxCellCoords = UActorPartitionSubsystem::FCellCoord::GetCellCoord(InRect.Max, InLevel, GridSize);

	for (int32 y = MinCellCoords.Y; y <= MaxCellCoords.Y; y++)
	{
		for (int32 x = MinCellCoords.X; x <= MaxCellCoords.X; x++)
		{
			UActorPartitionSubsystem::FCellCoord CellCoords(x, y, 0, InLevel);
			const FIntPoint Min = FIntPoint(CellCoords.X * GridSize, CellCoords.Y * GridSize);
			const FIntPoint Max = Min + FIntPoint(GridSize);
			FIntRect CellBounds(Min, Max);

			if (!InOperation(MoveTemp(CellCoords), MoveTemp(CellBounds)))
			{
				return;
			}
		}
	}
}

/**
 * FActorPartitionLevel
 */
class FActorPartitionLevel : public FBaseActorPartition
{
public:
	FActorPartitionLevel(UWorld* InWorld)
		: FBaseActorPartition(InWorld) 
	{
		 LevelRemovedFromWorldHandle = FWorldDelegates::LevelRemovedFromWorld.AddRaw(this, &FActorPartitionLevel::OnLevelRemovedFromWorld);
	}

	~FActorPartitionLevel()
	{
		FWorldDelegates::LevelRemovedFromWorld.Remove(LevelRemovedFromWorldHandle);
	}

	UActorPartitionSubsystem::FCellCoord GetActorPartitionHash(const FActorPartitionGetParams& GetParams) const override 
	{ 
		ULevel* SpawnLevel = GetSpawnLevel(GetParams.LevelHint, GetParams.LocationHint);
		return UActorPartitionSubsystem::FCellCoord(0, 0, 0, SpawnLevel);
	}
	
	APartitionActor* GetActor(const FActorPartitionIdentifier& InActorPartitionId, bool bInCreate, const UActorPartitionSubsystem::FCellCoord& InCellCoord, uint32 InGridSize, bool bInBoundsSearch, TFunctionRef<void(APartitionActor*)> InActorCreated) override
	{
		check(InCellCoord.Level);
		
		APartitionActor* FoundActor = nullptr;
		for (AActor* Actor : InCellCoord.Level->Actors)
		{
			if (APartitionActor* PartitionActor = Cast<APartitionActor>(Actor))
			{
				if (PartitionActor->IsA(InActorPartitionId.GetClass()) && (PartitionActor->GetGridGuid() == InActorPartitionId.GetGridGuid()))
				{
					FoundActor = PartitionActor;
					break;
				}
			}
		}
		
		if (!FoundActor && bInCreate)
		{
			FActorSpawnParameters SpawnParams;
			SpawnParams.OverrideLevel = InCellCoord.Level;
			FoundActor = CastChecked<APartitionActor>(World->SpawnActor(InActorPartitionId.GetClass(), nullptr, nullptr, SpawnParams));
			InActorCreated(FoundActor);
		}

		check(FoundActor || !bInCreate);
		return FoundActor;
	}


	void ForEachRelevantActor(const TSubclassOf<APartitionActor>& InActorClass, const FBox& IntersectionBounds, TFunctionRef<bool(APartitionActor*)>InOperation) const override
	{
		for (TActorIterator<APartitionActor> It(World, InActorClass); It; ++It)
		{
			if (!InOperation(*It))
			{
				return;
			}
		}
	}


private:
	void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
	{
		if (InWorld == World)
		{
			GetOnActorPartitionHashInvalidated().Broadcast(UActorPartitionSubsystem::FCellCoord(0, 0, 0, InLevel));
		}
	}

	ULevel* GetSpawnLevel(ULevel* InLevelHint, const FVector& InLocationHint) const
	{
		check(InLevelHint);
		ULevel* SpawnLevel = InLevelHint;
		return SpawnLevel;
	}

	FDelegateHandle LevelRemovedFromWorldHandle;
};

/**
 * FActorPartitionWorldPartition
 */
class FActorPartitionWorldPartition : public FBaseActorPartition
{
public:
	FActorPartitionWorldPartition(UWorld* InWorld)
		: FBaseActorPartition(InWorld)
	{
		check(InWorld->GetWorldPartition());
	}

	UActorPartitionSubsystem::FCellCoord GetActorPartitionHash(const FActorPartitionGetParams& GetParams) const override 
	{
		uint32 GridSize = GetParams.GridSize > 0 ? GetParams.GridSize : GetParams.ActorClass->GetDefaultObject<APartitionActor>()->GetDefaultGridSize(World);

		return UActorPartitionSubsystem::FCellCoord::GetCellCoord(GetParams.LocationHint, World->PersistentLevel, GridSize);
	}

	virtual APartitionActor* GetActor(const FActorPartitionIdentifier& InActorPartitionId, bool bInCreate, const UActorPartitionSubsystem::FCellCoord& InCellCoord, uint32 InGridSize, bool bInBoundsSearch, TFunctionRef<void(APartitionActor*)> InActorCreated)
	{
		APartitionActor* FoundActor = nullptr;
		bool bUnloadedActorExists = false;
		FGuid ContentBundleGuid = UContentBundleEngineSubsystem::Get()->GetEditingContentBundleGuid();

		auto FindActor = [&FoundActor, &bUnloadedActorExists, &ContentBundleGuid, InCellCoord, InActorPartitionId, InGridSize, ThisWorld = World](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			const FWorldPartitionActorDesc* ActorDesc = ActorDescInstance->GetActorDesc();
			check(ActorDesc->GetActorNativeClass()->IsChildOf(InActorPartitionId.GetClass()));
			const FPartitionActorDesc* PartitionActorDesc = (const FPartitionActorDesc*)ActorDesc;

			if ((PartitionActorDesc->GridIndexX == InCellCoord.X) &&
				(PartitionActorDesc->GridIndexY == InCellCoord.Y) &&
				(PartitionActorDesc->GridIndexZ == InCellCoord.Z) &&
				(PartitionActorDesc->GridSize == InGridSize) &&
				(PartitionActorDesc->GridGuid == InActorPartitionId.GetGridGuid()) &&
				(FActorPartitionContextHash::Get(ActorDescInstance->GetContentBundleGuid(), ThisWorld, ActorDescInstance->GetDataLayerInstanceNames().ToArray()) == InActorPartitionId.GetContextHash()))
			{
				AActor* DescActor = ActorDescInstance->GetActor();

				if (!DescActor)
				{
					// Actor exists but is not loaded
					bUnloadedActorExists = true;
					return false;
				}

				// Skip invalid actors because they will be renamed out of the way later
				if (IsValidChecked(DescActor))
				{
					FoundActor = CastChecked<APartitionActor>(DescActor);
					return false;
				}
			}
			return true;
		};

		UWorldPartition* WorldPartition = InCellCoord.Level->GetWorldPartition();
		check(WorldPartition);

		FBox CellBounds = UActorPartitionSubsystem::FCellCoord::GetCellBounds(InCellCoord, InGridSize);
		if (bInBoundsSearch)
		{
			FWorldPartitionHelpers::ForEachIntersectingActorDescInstance(WorldPartition, CellBounds, InActorPartitionId.GetClass(), FindActor);
		}
		else
		{
			FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, InActorPartitionId.GetClass(), FindActor);
		}
				
		if (bUnloadedActorExists)
		{
			return nullptr;
		}

		if (!FoundActor && bInCreate)
		{
			const FString ActorName = APartitionActor::GetActorName(InCellCoord.Level->GetTypedOuter<UWorld>(), InActorPartitionId, InGridSize, InCellCoord.X, InCellCoord.Y, InCellCoord.Z);

			FActorSpawnParameters SpawnParams;
			SpawnParams.OverrideLevel = InCellCoord.Level;
			SpawnParams.Name = *ActorName;
			SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Required_Fatal;

			// Handle the case where the actor already exists, but is in the undo stack (was deleted)
			if (UObject* ExistingObject = StaticFindObject(nullptr, InCellCoord.Level, *SpawnParams.Name.ToString()))
			{
				AActor* ExistingActor = CastChecked<AActor>(ExistingObject);
				check(!IsValidChecked(ExistingActor));
				ExistingActor->Modify();
				// Don't go through AActor::Rename here because we aren't changing outers (the actor's level). We just want to rename that actor 
				// out of the way so we can spawn the new one in the exact same package, keeping the package name intact.
				ExistingActor->UObject::Rename(nullptr, nullptr, REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
				
				// Reuse ActorGuid so that ActorDesc can be updated on save
				SpawnParams.OverrideActorGuid = ExistingActor->GetActorGuid();
			}
						
			FVector CellCenter(CellBounds.GetCenter());
			FoundActor = CastChecked<APartitionActor>(World->SpawnActor(InActorPartitionId.GetClass(), &CellCenter, nullptr, SpawnParams));
			FoundActor->SetGridSize(InGridSize);
			FoundActor->SetLockLocation(true);
			
			InActorCreated(FoundActor);

			APartitionActor::SetLabelForActor(FoundActor, InActorPartitionId, InGridSize, InCellCoord.X, InCellCoord.Y, InCellCoord.Z);
		}

		check(FoundActor || !bInCreate);
		return FoundActor;
	}

	void ForEachRelevantActor(const TSubclassOf<APartitionActor>& InActorClass, const FBox& IntersectionBounds, TFunctionRef<bool(APartitionActor*)>InOperation) const override
	{
		UActorPartitionSubsystem* ActorSubsystem = World->GetSubsystem<UActorPartitionSubsystem>();
		FActorPartitionGridHelper::ForEachIntersectingCell(InActorClass, IntersectionBounds, World->PersistentLevel, [&ActorSubsystem, &InActorClass, &IntersectionBounds, &InOperation](const UActorPartitionSubsystem::FCellCoord& InCellCoord, const FBox& InCellBounds) {

			if (InCellBounds.Intersect(IntersectionBounds))
			{
				const bool bCreate = false;
				if (auto PartitionActor = ActorSubsystem->GetActor(InActorClass, InCellCoord, bCreate))
				{
					return InOperation(PartitionActor);
				}
			}
			return true;
			});
	}
};

#endif // WITH_EDITOR

UActorPartitionSubsystem::UActorPartitionSubsystem()
{}

bool UActorPartitionSubsystem::IsLevelPartition() const
{
	return !UWorld::IsPartitionedWorld(GetWorld());
}

bool UActorPartitionSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return Super::DoesSupportWorldType(WorldType) || WorldType == EWorldType::Inactive || WorldType == EWorldType::EditorPreview;
}

#if WITH_EDITOR
void UActorPartitionSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency<UWorldPartitionSubsystem>();
	
	// Will need to register to WorldPartition setup changes events here...
	InitializeActorPartition();
}

/** Implement this for deinitialization of instances of the system */
void UActorPartitionSubsystem::Deinitialize()
{
	UninitializeActorPartition();
}

void UActorPartitionSubsystem::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	if (InWorldPartition->IsMainWorldPartition())
	{
		UninitializeActorPartition();
		InitializeActorPartition();
	}
}

void UActorPartitionSubsystem::ForEachRelevantActor(const TSubclassOf<APartitionActor>& InActorClass, const FBox& IntersectionBounds, TFunctionRef<bool(APartitionActor*)>InOperation) const
{
	if (ActorPartition)
	{
		ActorPartition->ForEachRelevantActor(InActorClass, IntersectionBounds, InOperation);
	}
}

void UActorPartitionSubsystem::OnActorPartitionHashInvalidated(const FCellCoord& Hash)
{
	PartitionedActors.Remove(Hash);
}

void UActorPartitionSubsystem::InitializeActorPartition()
{
	check(!ActorPartition);

	if (IsLevelPartition())
	{
		ActorPartition.Reset(new FActorPartitionLevel(GetWorld()));

		// Specific use case where map is Converted to World Partition from a non World Partition template
		if (GetWorld()->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
		{
			GetWorld()->OnWorldPartitionInitialized().AddUObject(this, &UActorPartitionSubsystem::OnWorldPartitionInitialized);
		}
	}
	else
	{
		ActorPartition.Reset(new FActorPartitionWorldPartition(GetWorld()));
	}
	ActorPartitionHashInvalidatedHandle = ActorPartition->GetOnActorPartitionHashInvalidated().AddUObject(this, &UActorPartitionSubsystem::OnActorPartitionHashInvalidated);
}

void UActorPartitionSubsystem::UninitializeActorPartition()
{
	PartitionedActors.Empty();
	if (ActorPartition)
	{
		ActorPartition->GetOnActorPartitionHashInvalidated().Remove(ActorPartitionHashInvalidatedHandle);
	}

	ActorPartition = nullptr;
	GetWorld()->OnWorldPartitionInitialized().RemoveAll(this);
}

APartitionActor* UActorPartitionSubsystem::GetActor(const FActorPartitionGetParams& GetParams)
{
	FCellCoord CellCoord = ActorPartition->GetActorPartitionHash(GetParams);
	return GetActor(GetParams.ActorClass, CellCoord, GetParams.bCreate, GetParams.GuidHint, GetParams.GridSize, GetParams.bBoundsSearch, GetParams.ActorCreatedCallback);
}

APartitionActor* UActorPartitionSubsystem::GetActor(const TSubclassOf<APartitionActor>& InActorClass, const FCellCoord& InCellCoords, bool bInCreate, const FGuid& InGuid, uint32 InGridSize, bool bInBoundsSearch, TFunctionRef<void(APartitionActor*)> InActorCreated)
{
	UWorld* World = GetWorld();
	FGuid ContentBundleGuid = UContentBundleEngineSubsystem::Get()->GetEditingContentBundleGuid();
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(World);
	const uint32 DataLayerEditorContextHash = DataLayerManager ? DataLayerManager->GetDataLayerEditorContextHash() : FDataLayerEditorContext::EmptyHash;
	FActorPartitionIdentifier ActorPartitionId(InActorClass, InGuid, FActorPartitionContextHash::Get(ContentBundleGuid, DataLayerEditorContextHash));
	return GetActor(ActorPartitionId, InCellCoords, bInCreate, InGridSize, bInBoundsSearch, InActorCreated);
}

APartitionActor* UActorPartitionSubsystem::GetActor(const FActorPartitionIdentifier& InActorPartitionId, const FCellCoord& InCellCoords, bool bInCreate, uint32 InGridSize, bool bInBoundsSearch, TFunctionRef<void(APartitionActor*)> InActorCreated)
{
	UWorld* World = GetWorld();
	const uint32 GridSize = InGridSize > 0 ? InGridSize : InActorPartitionId.GetClass()->GetDefaultObject<APartitionActor>()->GetDefaultGridSize(World);
	TMap<FActorPartitionIdentifier, TWeakObjectPtr<APartitionActor>>* ActorsPerId = PartitionedActors.Find(InCellCoords);
	APartitionActor* FoundActor = nullptr;
	if (!ActorsPerId)
	{
		FoundActor = ActorPartition->GetActor(InActorPartitionId, bInCreate, InCellCoords, GridSize, bInBoundsSearch, InActorCreated);
		if (FoundActor)
		{
			PartitionedActors.Add(InCellCoords).Add(InActorPartitionId, FoundActor);
		}
	}
	else
	{
		TWeakObjectPtr<APartitionActor>* ActorPtr = ActorsPerId->Find(InActorPartitionId);
		if (!ActorPtr || !ActorPtr->IsValid())
		{
			FoundActor = ActorPartition->GetActor(InActorPartitionId, bInCreate, InCellCoords, GridSize, bInBoundsSearch, InActorCreated);
			if (FoundActor)
			{
				if (!ActorPtr)
				{
					ActorsPerId->Add(InActorPartitionId, FoundActor);
				}
				else
				{
					*ActorPtr = FoundActor;
				}
			}
		}
		else
		{
			FoundActor = ActorPtr->Get();
		}
	}
	
	return FoundActor;
}

bool UActorPartitionSubsystem::MoveActorToDataLayers(APartitionActor* InActor, const TArray<UDataLayerInstance*>& InDataLayerInstances)
{
	auto ChangeActorExternalPackage = [](AActor* InActor, const UPackage* InOldActorPackage, UPackage* InNewActorPackage)
	{
		const bool bShouldDirty = true;
		const bool bLevelPackageWasDirty = InActor->GetLevel()->GetPackage()->IsDirty();

		InActor->SetPackageExternal(false, bShouldDirty);

		// Get all other dependant objects in the old actor package
		TArray<UObject*> DependantObjects;
		ForEachObjectWithPackage(InOldActorPackage, [&DependantObjects](UObject* Object)
		{
			if (!Cast<UDeletedObjectPlaceholder>(Object) && !Cast<AActor>(Object))
			{
				DependantObjects.Add(Object);
			}
			return true;
		}, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage); // Skip garbage objects (like child actors destroyed when de-externalizing the actor)

		InActor->SetPackageExternal(true, bShouldDirty, InNewActorPackage);
		check(InNewActorPackage == InActor->GetExternalPackage());

		// Move dependant objects into the new actor package
		for (UObject* DependantObject : DependantObjects) //-V1078
		{
			DependantObject->Rename(nullptr, InNewActorPackage, REN_NonTransactional | REN_DontCreateRedirectors | REN_DoNotDirty);
		}

		// Restore level package dirty flag
		if (!bLevelPackageWasDirty)
		{
			InActor->GetLevel()->GetPackage()->SetDirtyFlag(false);
		}
	};

	if (!InActor->IsPartitionActorNameAffectedByDataLayers())
	{
		UE_LOGF(LogWorldPartition, Warning, "Failed to move actor %ls to Data Layers: Use UDataLayerEditorSubsystem::MoveActorToDataLayers instead.", *InActor->GetName());
		return false;
	}

	UWorld* World = InActor->GetWorld();
	TArray<FName> DataLayerInstanceNames;
	Algo::Transform(InDataLayerInstances, DataLayerInstanceNames, [](UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->GetFName(); });
	const uint32 DataLayerEditorContextHash = FDataLayerEditorContext(World, DataLayerInstanceNames).GetHash();
	const FActorPartitionIdentifier ActorPartitionId(InActor->GetClass(), InActor->GetGridGuid(), FActorPartitionContextHash::Get(InActor->GetContentBundleGuid(), DataLayerEditorContextHash));
	const TUniquePtr<FWorldPartitionActorDesc> ActorDesc = InActor->CreateActorDesc();
	const FPartitionActorDesc* PartitionActorDesc = (const FPartitionActorDesc*)ActorDesc.Get();
	const UActorPartitionSubsystem::FCellCoord CellCoord(PartitionActorDesc->GridIndexX, PartitionActorDesc->GridIndexY, PartitionActorDesc->GridIndexZ, InActor->GetLevel());
	const FString NewActorName = APartitionActor::GetActorName(CellCoord.Level->GetTypedOuter<UWorld>(), ActorPartitionId, InActor->GetGridSize(), CellCoord.X, CellCoord.Y, CellCoord.Z);
	const FString& OldActorName = InActor->GetName();

	// Test if there's nothing to do
	if (OldActorName == NewActorName)
	{
		return true;
	}

	// Test moving from/to new External Data Layer
	const UExternalDataLayerAsset* OldExternalDataLayerAsset = InActor->GetExternalDataLayerAsset();
	const UDataLayerInstance* const* ExternalDataLayerInstance = Algo::FindByPredicate(InDataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsA<UExternalDataLayerInstance>(); });
	const UExternalDataLayerInstance* NewExternalDataLayerInstance = ExternalDataLayerInstance ? Cast<UExternalDataLayerInstance>(*ExternalDataLayerInstance) : nullptr;
	const UExternalDataLayerAsset* NewExternalDataLayerAsset = NewExternalDataLayerInstance ? NewExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr;
	constexpr bool bAllowNonUserManaged = true;
	FMoveToExternalDataLayerParams Params(NewExternalDataLayerInstance, bAllowNonUserManaged);
	if ((OldExternalDataLayerAsset != NewExternalDataLayerAsset))
	{
		FText FailureReason;
		if (!FExternalDataLayerHelper::CanMoveActorsToExternalDataLayer({ InActor }, Params, &FailureReason))
		{
			UE_LOGF(LogWorldPartition, Warning, "Can't move actor %ls to External Data Layer. %ls", *InActor->GetName(), *FailureReason.ToString());
			return false;
		}
	}

	// Find any existing actor with this setup
	constexpr bool bCreate = false;
	constexpr bool bBoundsSearch = false;
	if (APartitionActor* ExistingActor = GetActor(ActorPartitionId, CellCoord, bCreate, InActor->GetGridSize(), bBoundsSearch); ExistingActor && (ExistingActor != InActor))
	{
		UE_LOGF(LogEngine, Warning, "Failed to move actor %ls to Data Layers: Another partition actor %ls already exists.", *InActor->GetName(), *ExistingActor->GetName());
		return false;
	}

	// Update PartitionedActors
	TMap<FActorPartitionIdentifier, TWeakObjectPtr<APartitionActor>>* ActorsPerId = PartitionedActors.Find(CellCoord);
	if (ActorsPerId)
	{
		for (auto It = ActorsPerId->CreateIterator(); It; ++It)
		{
			if (It.Value().Get() == InActor)
			{
				It.RemoveCurrent();
				break;
			}
		}
	}
	else
	{
		ActorsPerId = &PartitionedActors.Add(CellCoord);
	}
	ActorsPerId->Add(ActorPartitionId, InActor);

	// Handle the case where the actor already exists, but is in the undo stack (was deleted)
	if (UObject* ExistingObject = StaticFindObject(nullptr, CellCoord.Level, *NewActorName))
	{
		AActor* ExistingActor = CastChecked<AActor>(ExistingObject);
		check(!IsValidChecked(ExistingActor));
		ExistingActor->Modify();
		// Don't go through AActor::Rename here because we aren't changing outers (the actor's level). 
		// We just want to rename that actor out of the way.
		ExistingActor->UObject::Rename(nullptr, nullptr, REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional);
	}

	// Create new actor package
	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> NewActorPath;
	NewActorPath += CellCoord.Level->GetPathName();
	NewActorPath += TEXT(".");
	NewActorPath += NewActorName;
	UPackage* NewActorPackage = ULevel::CreateActorPackage(CellCoord.Level->GetPackage(), CellCoord.Level->GetActorPackagingScheme(), *NewActorPath);
	
	// Swap content from the old external package to the new one
	const UPackage* OldActorPackage = InActor->GetExternalPackage();
	ChangeActorExternalPackage(InActor, OldActorPackage, NewActorPackage);

	// Update name and label
	InActor->Rename(*NewActorName, nullptr, REN_NonTransactional | REN_DontCreateRedirectors | REN_DoNotDirty);
	APartitionActor::SetLabelForActor(InActor, ActorPartitionId, InActor->GetGridSize(), CellCoord.X, CellCoord.Y, CellCoord.Z);

	// Move to new Data Layers:
	// First, remove all non-EDL layers
	const bool bIncludeExternalDataLayerAsset = false;
	for (const UDataLayerAsset* DataLayerAsset : InActor->GetDataLayerAssets(bIncludeExternalDataLayerAsset))
	{
		FAssignActorDataLayer::RemoveDataLayerAsset(InActor, DataLayerAsset);
	}
	// Second, move to new External Data Layer if any or different. Subsequent valid data layers should be in that EDL, and the EDL must be added first for package paths to allow adding the DLs.
	if (OldExternalDataLayerAsset != NewExternalDataLayerAsset)
	{
		verify(FExternalDataLayerHelper::MoveActorsToExternalDataLayer({ InActor }, Params));
	}
	// Finally, apply the rest of the layers
	for (const UDataLayerInstance* DataLayerInstance : InDataLayerInstances)
	{
		if (const UDataLayerAsset* DataLayerAsset = !DataLayerInstance->IsA<UExternalDataLayerInstance>() ? DataLayerInstance->GetAsset() : nullptr)
		{
			FAssignActorDataLayer::AddDataLayerAsset(InActor, DataLayerAsset);
		}
	}

	return true;
}

#endif // WITH_EDITOR

