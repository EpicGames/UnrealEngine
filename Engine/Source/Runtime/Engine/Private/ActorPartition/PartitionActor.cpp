// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPartition/PartitionActor.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "String/ParseTokens.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(PartitionActor)

#if WITH_EDITOR
#include "Engine/Level.h"
#include "WorldPartition/ActorPartition/PartitionActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerEditorContext.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/World.h"
#endif

#define LOCTEXT_NAMESPACE "PartitionActor"

DEFINE_LOG_CATEGORY_STATIC(LogPartitionActor, Log, All);

APartitionActor::APartitionActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, GridSize(1)
#endif
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent0"));
	RootComponent = SceneComponent;
	RootComponent->Mobility = EComponentMobility::Static;
}

#if WITH_EDITOR
TUniquePtr<FWorldPartitionActorDesc> APartitionActor::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FPartitionActorDesc());
}

bool APartitionActor::IsUserManaged() const
{
	if (!Super::IsUserManaged())
	{
		return false;
	}

	check(GetLevel());
	if (GetLevel()->bIsPartitioned)
	{
		return false;
	}

	return true;
}

bool APartitionActor::ShouldIncludeGridSizeInName(UWorld* InWorld, const FActorPartitionIdentifier& InIdentifier) const
{
	return InWorld->GetWorldSettings()->bIncludeGridSizeInNameForPartitionedActors;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
uint32 APartitionActor::GetGridSize() const
{
	return GridSize;
} 

void APartitionActor::SetGridSize(uint32 InGridSize)
{
	if (InGridSize == 0)
	{
		UE_LOGF(LogPartitionActor, Error, "APartitionActor::SetGridSize() called for actor %ls with grid size == 0. Grid size must be greater than zero.", *GetName());
		InGridSize = 1;
	}
	GridSize = InGridSize;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FString APartitionActor::GetActorName(UWorld* World, const UClass* Class, const FGuid& GridGuid, const FActorPartitionIdentifier& ActorPartitionId, uint32 GridSize, int32 CellCoordsX, int32 CellCoordsY, int32 CellCoordsZ, uint32 ContextHash)
{
	return GetActorName(World, ActorPartitionId, GridSize, CellCoordsX, CellCoordsY, CellCoordsZ);
}

FString APartitionActor::GetActorName(UWorld* World, const FActorPartitionIdentifier& ActorPartitionId, uint32 GridSize, int32 CellCoordsX, int32 CellCoordsY, int32 CellCoordsZ)
{
	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> ActorNameBuilder;

	ActorNameBuilder += ActorPartitionId.GetClass()->GetName();
	ActorNameBuilder += TEXT("_");

	if (ActorPartitionId.GetGridGuid().IsValid())
	{
		ActorNameBuilder += ActorPartitionId.GetGridGuid().ToString(EGuidFormats::Base36Encoded);
		ActorNameBuilder += TEXT("_");
	}

	if (ActorPartitionId.GetClass()->GetDefaultObject<APartitionActor>()->ShouldIncludeGridSizeInName(World, ActorPartitionId))
	{
		ActorNameBuilder += FString::Printf(TEXT("%d_"), GridSize);
	}

	ActorNameBuilder += FString::Printf(TEXT("%d_%d_%d"), CellCoordsX, CellCoordsY, CellCoordsZ);

	if (ActorPartitionId.GetContextHash() != FActorPartitionIdentifier::EmptyContextHash)
	{
		ActorNameBuilder += FString::Printf(TEXT("_%X"), ActorPartitionId.GetContextHash());
	}

	return ActorNameBuilder.ToString();
}

void APartitionActor::SetLabelForActor(APartitionActor* Actor, const FActorPartitionIdentifier& ActorPartitionId, uint32 GridSize, int32 CellCoordsX, int32 CellCoordsY, int32 CellCoordsZ)
{
	// Once actor is created, update its label
	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> ActorLabelBuilder;
	ActorLabelBuilder += FString::Printf(TEXT("%s"), *ActorPartitionId.GetClass()->GetName());
	if (Actor->ShouldIncludeGridSizeInLabel())
	{
		ActorLabelBuilder += FString::Printf(TEXT("_%u"), GridSize);
	}
	ActorLabelBuilder += FString::Printf(TEXT("_%d_%d_%d"), CellCoordsX, CellCoordsY, CellCoordsZ);
	if (ActorPartitionId.GetContextHash() != FActorPartitionIdentifier::EmptyContextHash)
	{
		ActorLabelBuilder += FString::Printf(TEXT("_%X"), ActorPartitionId.GetContextHash());
	}
	Actor->SetActorLabel(*ActorLabelBuilder);
}
#endif

// Parse the actor name to confirm that it's a partitioned actor or not (see APartitionActor::GetActorName).
bool APartitionActor::IsPartitioned(const APartitionActor* Actor)
{
#if WITH_EDITOR
	int32 NumSeparators = 0;
	bool bValidClass = false;
	bool bValidGridSize = !Actor->ShouldIncludeGridSizeInName(Actor->GetWorld(), { Actor->GetClass(), Actor->GetGridGuid(), 0 });
	bool bValidGridGuid = !Actor->GetGridGuid().IsValid();
	int32 NumValidCoords = 0;

	UE::String::ParseTokens(Actor->GetName(), TEXT("_"), 
		[&Actor, &NumSeparators, &bValidClass, &bValidGridSize, &bValidGridGuid, &NumValidCoords](FStringView InToken)
		{
			switch (NumSeparators)
			{
			case 0:
				bValidClass = Actor->GetClass()->GetName() == InToken;
				break;

			case 1:
				if (!bValidGridGuid)
				{
					FGuid Guid;
					bValidGridGuid = FGuid::ParseExact(InToken, EGuidFormats::Base36Encoded, Guid) && (Guid == Actor->GetGridGuid());
					break;
				}
			[[fallthrough]];

			case 2:
				if (bValidGridGuid && !bValidGridSize)
				{
					bValidGridSize = FCString::Atoi(*FString(InToken)) == Actor->GetGridSize();
					break;
				}
			[[fallthrough]];

			default:
				NumValidCoords++;
				break;
			}

			NumSeparators++;
		});

	return bValidClass && bValidGridSize && bValidGridGuid && ((NumValidCoords == 3) || (NumValidCoords == 4));
#else
	return false;
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
