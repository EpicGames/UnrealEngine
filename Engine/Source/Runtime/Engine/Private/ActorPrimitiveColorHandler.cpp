// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "GameFramework/Actor.h"
#include "Misc/LazySingleton.h"
#include "EngineUtils.h"

#define LOCTEXT_NAMESPACE "ActorColoration"

FActorPrimitiveColorHandler::FActorPrimitiveColorHandler()
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	InitActivePrimitiveColorHandler();
#endif
}

void FActorPrimitiveColorHandler::InitActivePrimitiveColorHandler()
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	ActivePrimitiveColorHandlerName = NAME_None;
	ActivePrimitiveColorHandler = nullptr;
#endif
}

FActorPrimitiveColorHandler& FActorPrimitiveColorHandler::Get()
{
	return TLazySingleton<FActorPrimitiveColorHandler>::Get();
}

void FActorPrimitiveColorHandler::RegisterPrimitiveColorHandler(FActorPrimitiveColorHandler::FPrimitiveColorHandler& PrimitiveColorHandler)
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	check(!Handlers.Contains(PrimitiveColorHandler.HandlerName));
	Handlers.Add(PrimitiveColorHandler.HandlerName, PrimitiveColorHandler);
#endif
}

void FActorPrimitiveColorHandler::RegisterPrimitiveColorHandler(FName InHandlerName, const FText& InHandlerText, const FGetColorFunc& InHandlerFunc, const FActivateFunc& InActivateFunc, const FText& InHandlerToolTipText)
{
	FPrimitiveColorHandler Handler{ InHandlerName, InHandlerText, InHandlerToolTipText, true, InHandlerFunc, InActivateFunc };
	RegisterPrimitiveColorHandler(Handler);
}

void FActorPrimitiveColorHandler::RegisterPrimitiveColorHandler(FName InHandlerName, const FText& InHandlerText, bool bInAvailalbleInEditor, const FGetColorFunc& InHandlerFunc, const FActivateFunc& InActivateFunc, const FText& InHandlerToolTipText)
{
	FPrimitiveColorHandler Handler{ InHandlerName, InHandlerText, InHandlerToolTipText, bInAvailalbleInEditor, InHandlerFunc, InActivateFunc };
	RegisterPrimitiveColorHandler(Handler);
}

void FActorPrimitiveColorHandler::UnregisterPrimitiveColorHandler(FName InHandlerName)
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	check(!InHandlerName.IsNone());
	check(Handlers.Contains(InHandlerName));
	Handlers.Remove(InHandlerName);
	
	if (InHandlerName == ActivePrimitiveColorHandlerName)
	{
		if (ActivePrimitiveColorHandler && ActivePrimitiveColorHandler->DeactivateFunc.IsSet())
		{
			ActivePrimitiveColorHandler->DeactivateFunc();
		}

		InitActivePrimitiveColorHandler();
	}

#endif
}

bool FActorPrimitiveColorHandler::SetActivePrimitiveColorHandler(FName InHandlerName, UWorld* InWorld)
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	FPrimitiveColorHandler* NewActivePrimitiveColorHandler = Handlers.Find(InHandlerName);
	if (NewActivePrimitiveColorHandler != ActivePrimitiveColorHandler)
	{
		if (ActivePrimitiveColorHandler && ActivePrimitiveColorHandler->DeactivateFunc.IsSet())
		{
			ActivePrimitiveColorHandler->DeactivateFunc();
		}

		ActivePrimitiveColorHandler = NewActivePrimitiveColorHandler;
		ActivePrimitiveColorHandlerName = ActivePrimitiveColorHandler ? InHandlerName : NAME_None;

		if (ActivePrimitiveColorHandler && ActivePrimitiveColorHandler->ActivateFunc.IsSet())
		{
			ActivePrimitiveColorHandler->ActivateFunc();
		}

		RefreshPrimitiveColorHandler(InHandlerName, InWorld);
		return true;
	}
#endif

	return false;
}

void FActorPrimitiveColorHandler::RefreshPrimitiveColorHandler(FName InHandlerName, UWorld* InWorld)
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (ActivePrimitiveColorHandlerName == InHandlerName)
	{
		for (TActorIterator<AActor> It(InWorld); It; ++It)
		{
			It->ForEachComponent<UPrimitiveComponent>(false, [this](UPrimitiveComponent* PrimitiveComponent)
			{
				if (PrimitiveComponent->IsRegistered())
				{
					PrimitiveComponent->PushPrimitiveColorToProxy(GetPrimitiveColor(PrimitiveComponent));
				}
			});
		}
	}
#endif
}

void FActorPrimitiveColorHandler::RefreshPrimitiveColorHandler(FName InHandlerName, const TArray<AActor*>& InActors)
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (ActivePrimitiveColorHandlerName == InHandlerName)
	{
		for (AActor* Actor : InActors)
		{
			Actor->ForEachComponent<UPrimitiveComponent>(false, [this](UPrimitiveComponent* PrimitiveComponent)
			{
				if (PrimitiveComponent->IsRegistered())
				{
					PrimitiveComponent->PushPrimitiveColorToProxy(GetPrimitiveColor(PrimitiveComponent));
				}
			});
		}
	}
#endif
}

void FActorPrimitiveColorHandler::RefreshPrimitiveColorHandler(FName InHandlerName, const TArray<UPrimitiveComponent*>& InPrimitiveComponents)
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (ActivePrimitiveColorHandlerName == InHandlerName)
	{
		for (UPrimitiveComponent* PrimitiveComponent : InPrimitiveComponents)
		{
			if (PrimitiveComponent && PrimitiveComponent->IsRegistered())
			{
				PrimitiveComponent->PushPrimitiveColorToProxy(GetPrimitiveColor(PrimitiveComponent));
			}
		}
	}
#endif
}

FName FActorPrimitiveColorHandler::GetActivePrimitiveColorHandler() const
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	return ActivePrimitiveColorHandlerName;
#else
	return NAME_None;
#endif
}

FText FActorPrimitiveColorHandler::GetActivePrimitiveColorHandlerDisplayName() const
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	return ActivePrimitiveColorHandler != nullptr ? ActivePrimitiveColorHandler->HandlerText : FText();
#else
	return FText();
#endif
}

void FActorPrimitiveColorHandler::GetRegisteredPrimitiveColorHandlers(TArray<FPrimitiveColorHandler>& OutPrimitiveColorHandlers) const
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	Handlers.GenerateValueArray(OutPrimitiveColorHandlers);
#endif
}

FLinearColor FActorPrimitiveColorHandler::GetPrimitiveColor(const UPrimitiveComponent* InPrimitiveComponent) const
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	return ActivePrimitiveColorHandler != nullptr ? ActivePrimitiveColorHandler->GetColorFunc(InPrimitiveComponent) : FLinearColor::White;
#else
	return FLinearColor::White;
#endif
}

#undef LOCTEXT_NAMESPACE