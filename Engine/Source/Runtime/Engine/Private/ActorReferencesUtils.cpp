// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorReferencesUtils.h"
#include "GameFramework/Actor.h"
#include "Algo/Transform.h"
#include "VerseVM/Inline/VVMCellInline.h"

class FArchiveGatherExternalActorRefs : public FArchiveUObject
{
public:
	using FActorReference = ActorsReferencesUtils::FActorReference;

	struct FActorReferenceKeyFuncs : DefaultKeyFuncs<FActorReference>
	{
		template<typename ComparableKey>
		static FORCEINLINE bool Matches(KeyInitType A, ComparableKey B)
		{
			return A.Actor == B.Actor;
		}

		static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
		{
			return GetTypeHash(Key.Actor);
		}
	};

	using FActorReferenceSet = TSet<FActorReference, FActorReferenceKeyFuncs>;

	FArchiveGatherExternalActorRefs(UObject* InRoot, EObjectFlags InRequiredFlags, bool bInRecursive, FActorReferenceSet& InActorReference)
		: Root(InRoot)
		, ActorReferences(InActorReference)
		, EditorOnlyStack(0)
		, RequiredFlags(InRequiredFlags)
		, bRecursive(bInRecursive)
	{
		// Don't gather transient actor references
		SetIsPersistent(true);

		// Don't trigger serialization of compilable assets
		SetShouldSkipCompilingAssets(true);

		// We are a collector and don't modify the objects we visit
		SetIsSaving(true);

		ArIgnoreOuterRef = true;
		ArIsObjectReferenceCollector = true;
		ArShouldSkipBulkData = true;

		SubObjectsState.Add(Root, EReferenceState::Runtime);

		Root->Serialize(*this);
	}

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		if (Obj && !Obj->IsTemplate() && !Obj->HasAnyFlags(RF_Transient))
		{
			EReferenceState& ExistingReferenceState = SubObjectsState.FindOrAdd(Obj, EReferenceState::Unset);
			const EReferenceState ReferenceState = EditorOnlyStack > 0 ? EReferenceState::EditorOnly : EReferenceState::Runtime;

			if (ExistingReferenceState < ReferenceState)
			{
				ExistingReferenceState = ReferenceState;

				HandleObjectReference(Obj);

				if (bRecursive || Obj->IsInOuter(Root))
				{
					Obj->Serialize(*this);
				}
			}
		}
		return *this;
	}

	virtual FArchive& operator<<(FWeakObjectPtr& Value) override
	{
		if (UObject* Object = Value.Get())
		{
			return *this << Object;
		}
		return *this;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Cell) override
	{
		AutoRTFM::UnreachableIfClosed();

		if (Cell && Cell->GetCppClassInfo()->InstancedCell)
		{
			bool bIsAlreadyInSet = false;
			VisitedCells.Add(Cell, &bIsAlreadyInSet);
			if (!bIsAlreadyInSet)
			{
				Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
				Cell->Serialize(Context, *this);
			}
		}
		return *this;
	}
#endif

private:
	virtual void PushSerializedProperty(FProperty* InProperty, const bool bIsEditorOnlyProperty) override
	{
		FArchiveUObject::PushSerializedProperty(InProperty, bIsEditorOnlyProperty);
		EditorOnlyStack += InProperty->IsEditorOnlyProperty() ? 1 : 0;
	}

	virtual void PopSerializedProperty(FProperty* InProperty, const bool bIsEditorOnlyProperty) override
	{
		FArchiveUObject::PopSerializedProperty(InProperty, bIsEditorOnlyProperty);
		EditorOnlyStack -= InProperty->IsEditorOnlyProperty() ? 1 : 0;
		check(EditorOnlyStack >= 0);
	}

	void HandleObjectReference(UObject* Obj)
	{
		AActor* Actor = Cast<AActor>(Obj);

		if (!Actor)
		{
			Actor = Obj->GetTypedOuter<AActor>();
		}

		if (Actor)
		{
			AActor* TopParentActor = Actor;
			while (TopParentActor->GetParentActor())
			{
				TopParentActor = TopParentActor->GetParentActor();
			}

			check(TopParentActor);

			if (((RequiredFlags == RF_NoFlags) || TopParentActor->HasAnyFlags(RequiredFlags)) && (TopParentActor != Root))
			{
				bool bWasAlreadyInSetPtr;
				FActorReference TopParentActorReference;
				TopParentActorReference.Actor = TopParentActor;
				FActorReference& ActorReference = ActorReferences.FindOrAdd(TopParentActorReference, &bWasAlreadyInSetPtr);

				const bool bIsEditorOnly = EditorOnlyStack > 0;
				if (!bWasAlreadyInSetPtr)
				{
					ActorReference.bIsEditorOnly = bIsEditorOnly;
				}
				else
				{
					ActorReference.bIsEditorOnly &= bIsEditorOnly;
				}
			}
		}
	}

	enum class EReferenceState : uint8
	{
		Unset = 0,
		EditorOnly = 1,
		Runtime = 2
	};

	UObject* Root;
	FActorReferenceSet& ActorReferences;
	TMap<UObject*, EReferenceState> SubObjectsState;
	TSet<Verse::VCell*> VisitedCells;
	int32 EditorOnlyStack;
	EObjectFlags RequiredFlags;
	bool bRecursive;
};

TArray<AActor*> ActorsReferencesUtils::GetExternalActorReferences(UObject* Root, bool bRecursive)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetActorReferences(Root, RF_HasExternalPackage, bRecursive);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TArray<AActor*> ActorsReferencesUtils::GetActorReferences(UObject* Root, EObjectFlags RequiredFlags, bool bRecursive)
{
	FGetActorReferencesParams Params = FGetActorReferencesParams(Root)
		.SetRequiredFlags(RequiredFlags)
		.SetRecursive(bRecursive);

	TArray<AActor*> Result;
	const TArray<FActorReference> ActorReferences = GetActorReferences(Params);
	Algo::Transform(ActorReferences, Result, [](const FActorReference& ActorReference) { return ActorReference.Actor; });
	return Result;
}

TArray<ActorsReferencesUtils::FActorReference> ActorsReferencesUtils::GetActorReferences(const FGetActorReferencesParams& InParams)
{
	FArchiveGatherExternalActorRefs::FActorReferenceSet Result;
	FArchiveGatherExternalActorRefs Ar(InParams.Root, InParams.RequiredFlags, InParams.bRecursive, Result);
	return Result.Array();
}