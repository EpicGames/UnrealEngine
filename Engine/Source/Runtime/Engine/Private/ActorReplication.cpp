// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "Engine/Engine.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Net/UnrealNetwork.h"
#include "Engine/ActorChannel.h"
#include "Engine/NetDriver.h"
#include "GameFramework/GameNetworkManager.h"
#include "NetworkingDistanceConstants.h"
#include "PhysicsReplication.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "DrawDebugHelpers.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Interfaces/Interface_ActorSubobject.h"
#include "Engine/NetConnection.h"
#include "Iris/IrisConfig.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Conditionals/ReplicationCondition.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Physics/Experimental/PhysScene_Chaos.h"

/*-----------------------------------------------------------------------------
	AActor networking implementation.
-----------------------------------------------------------------------------*/

//
// Static variables for networking.
//
namespace ActorReplication
{
	static bool		SavedbHidden;
	static AActor*	SavedOwner;
	static bool		SavedbRepPhysics;
	static ENetRole SavedRole;
}

static bool GbSendPhysicsAttachRep = true;
static FAutoConsoleVariableRef CVarSendPhysicsAttachRep(TEXT("Net.RepMovement.SendPhysicsAttachRep"), GbSendPhysicsAttachRep, TEXT("Whether to replicate attachment replication info when the actor is simulating physics"), ECVF_ReadOnly);

using namespace UE::Net;

float AActor::GetNetPriority(const FVector& ViewPos, const FVector& ViewDir, AActor* Viewer, AActor* ViewTarget, UActorChannel* InChannel, float Time, bool bLowBandwidth)
{
	if (bNetUseOwnerRelevancy && Owner)
	{
		// If we should use our owner's priority, pass it through
		return Owner->GetNetPriority(ViewPos, ViewDir, Viewer, ViewTarget, InChannel, Time, bLowBandwidth);
	}

	if (ViewTarget && (this == ViewTarget || GetInstigator() == ViewTarget))
	{
		// If we're the view target or owned by the view target, use a high priority
		Time *= 4.f;
	}
	else if (!IsHidden() && GetRootComponent() != NULL)
	{
		// If this actor has a location, adjust priority based on location
		FVector Dir = GetActorLocation() - ViewPos;
		float DistSq = Dir.SizeSquared();

		// Adjust priority based on distance and whether actor is in front of viewer
		if ((ViewDir | Dir) < 0.f)
		{
			if (DistSq > NEARSIGHTTHRESHOLDSQUARED)
			{
				Time *= 0.2f;
			}
			else if (DistSq > CLOSEPROXIMITYSQUARED)
			{
				Time *= 0.4f;
			}
		}
		else if ((DistSq < FARSIGHTTHRESHOLDSQUARED) && (FMath::Square(ViewDir | Dir) > 0.5f * DistSq))
		{
			// Compute the amount of distance along the ViewDir vector. Dir is not normalized
			// Increase priority if we're being looked directly at
			Time *= 2.f;
		}
		else if (DistSq > MEDSIGHTTHRESHOLDSQUARED)
		{
			Time *= 0.4f;
		}
	}

	return NetPriority * Time;
}

float AActor::GetReplayPriority(const FVector& ViewPos, const FVector& ViewDir, class AActor* Viewer, AActor* ViewTarget, UActorChannel* const InChannel, float Time)
{
	if (ViewTarget && (this == ViewTarget || GetInstigator() == ViewTarget))
	{
		// If we're the view target or owned by the view target, use a high priority
		Time *= 10.0f;
	}
	else if (!IsHidden() && GetRootComponent() != NULL)
	{
		// If this actor has a location, adjust priority based on location
		FVector Dir = GetActorLocation() - ViewPos;
		float DistSq = Dir.SizeSquared();

		// Adjust priority based on distance
		if (DistSq < CLOSEPROXIMITYSQUARED)
		{
			Time *= 4.0f;
		}
		else if (DistSq < NEARSIGHTTHRESHOLDSQUARED)
		{
			Time *= 3.0f;
		}
		else if (DistSq < MEDSIGHTTHRESHOLDSQUARED)
		{
			Time *= 2.4f;
		}
		else if (DistSq < FARSIGHTTHRESHOLDSQUARED)
		{
			Time *= 0.8f;
		}
		else
		{
			Time *= 0.2f;
		}
	}

	// Use NetPriority here to be compatible with live networking.
	return NetPriority * Time;
}

bool AActor::GetNetDormancy(const FVector& ViewPos, const FVector& ViewDir, AActor* Viewer, AActor* ViewTarget, UActorChannel* InChannel, float Time, bool bLowBandwidth)
{
	// For now, per peer dormancy is not supported
	return false;
}

void AActor::PreNetReceive()
{
	ActorReplication::SavedbHidden = IsHidden();
	ActorReplication::SavedOwner = Owner;
	ActorReplication::SavedbRepPhysics = GetReplicatedMovement().bRepPhysics;
	ActorReplication::SavedRole = GetLocalRole();
}

void AActor::PostNetReceive()
{
	if (!bNetCheckedInitialPhysicsState)
	{
		// Initially we need to sync the state regardless of whether bRepPhysics has "changed" since it may not currently match IsSimulatingPhysics().
		SyncReplicatedPhysicsSimulation();
		ActorReplication::SavedbRepPhysics = GetReplicatedMovement().bRepPhysics;
		bNetCheckedInitialPhysicsState = true;
	}

	ExchangeB(bHidden, ActorReplication::SavedbHidden);
	Exchange(Owner, ActorReplication::SavedOwner);

	if (IsHidden() != ActorReplication::SavedbHidden)
	{
		SetActorHiddenInGame(ActorReplication::SavedbHidden);
	}
	if (Owner != ActorReplication::SavedOwner)
	{
		SetOwner(ActorReplication::SavedOwner);
	}

	if (GetLocalRole() != ActorReplication::SavedRole)
	{
		PostNetReceiveRole();
	}
}

void AActor::PostNetReceiveRole()
{
}

static TAutoConsoleVariable<int32> CVarDrawDebugRepMovement(TEXT("Net.RepMovement.DrawDebug"), 0, TEXT(""), ECVF_Default);

void AActor::OnRep_ReplicatedMovement()
{
	// Since ReplicatedMovement and AttachmentReplication are REPNOTIFY_Always (and OnRep_AttachmentReplication may call OnRep_ReplicatedMovement directly),
	// this check is needed since this can still be called on actors for which bReplicateMovement is false - for example, during fast-forward in replay playback.
	// When this happens, the values in ReplicatedMovement aren't valid, and must be ignored.
	if (!IsReplicatingMovement())
	{
		return;
	}

	const FRepMovement& LocalRepMovement = GetReplicatedMovement();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (CVarDrawDebugRepMovement->GetInt() > 0)
		{
			FColor DebugColor = FColor::Green;
			if (LocalRepMovement.bRepPhysics)
			{
				switch (GetPhysicsReplicationMode())
				{
				case EPhysicsReplicationMode::PredictiveInterpolation:
					DebugColor = FColor::Yellow;
					break;
				case EPhysicsReplicationMode::Resimulation:
					DebugColor = FColor::Red;
					break;
				case EPhysicsReplicationMode::Default:
					DebugColor = FColor::Cyan;
					break;
				case EPhysicsReplicationMode::None:
				default:
					DebugColor = FColor::Black;
					break;
				}
			}
			DrawDebugCapsule(GetWorld(), LocalRepMovement.Location, FMath::Max(GetSimpleCollisionHalfHeight(), 25.0f), FMath::Max(GetSimpleCollisionRadius(), 25.0f), LocalRepMovement.Rotation.Quaternion(), DebugColor, false, 1.f);
		}
#endif

	if (RootComponent)
	{
		if (ActorReplication::SavedbRepPhysics != LocalRepMovement.bRepPhysics)
		{
			// Turn on/off physics sim to match server.
			SyncReplicatedPhysicsSimulation();
		}

		// NOTE: This is only needed because ClusterUnion has a flag bHasReceivedTransform
		// which does not get updated until the component's transform is directly set.
		// Until that flag is set, its root particle will be in a disabled state and not
		// have any children, therefore replication will be dead in the water.
		RootComponent->OnReceiveReplicatedState(LocalRepMovement.Location, LocalRepMovement.Rotation.Quaternion(), LocalRepMovement.LinearVelocity, LocalRepMovement.AngularVelocity);

		if (LocalRepMovement.bRepPhysics)
		{
			// Sync physics state
#if DO_GUARD_SLOW
			if (!RootComponent->IsSimulatingPhysics())
			{
				UE_LOGF(LogNet, Warning, "IsSimulatingPhysics() returned false during physics replication for %ls", *GetName());
			}
#endif
			// If we are welded we just want the parent's update to move us.
			UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(RootComponent);
			if (!RootPrimComp || !RootPrimComp->IsWelded())
			{
				PostNetReceivePhysicState();
			}
		}
		else
		{
			// Attachment trumps global position updates, see GatherCurrentMovement().
			if (!RootComponent->GetAttachParent())
			{
				if (GetLocalRole() == ROLE_SimulatedProxy)
				{
#if ENABLE_NAN_DIAGNOSTIC
					if (LocalRepMovement.Location.ContainsNaN())
					{
						logOrEnsureNanError(TEXT("AActor::OnRep_ReplicatedMovement found NaN in ReplicatedMovement.Location"));
					}
					if (LocalRepMovement.Rotation.ContainsNaN())
					{
						logOrEnsureNanError(TEXT("AActor::OnRep_ReplicatedMovement found NaN in ReplicatedMovement.Rotation"));
					}
#endif

					PostNetReceiveVelocity(LocalRepMovement.LinearVelocity);
					PostNetReceiveLocationAndRotation();
				}
			}
		}
	}
}

void AActor::PostNetReceiveLocationAndRotation()
{
	const FRepMovement& LocalRepMovement = GetReplicatedMovement();
	FVector NewLocation = FRepMovement::RebaseOntoLocalOrigin(LocalRepMovement.Location, this);

	if( RootComponent && RootComponent->IsRegistered() && (NewLocation != GetActorLocation() || LocalRepMovement.Rotation != GetActorRotation()) )
	{
		SetActorLocationAndRotation(NewLocation, LocalRepMovement.Rotation, /*bSweep=*/ false);
	}
}

void AActor::PostNetReceiveVelocity(const FVector& NewVelocity)
{
	const FPhysicsPredictionSettings& PhysicsPredictionSettings = UPhysicsSettings::Get()->PhysicsPrediction;
	if (PhysicsPredictionSettings.bEnablePhysicsPrediction)
	{
		// required to keep client deterministic with server simulation
		if (RootComponent && RootComponent->IsRegistered() && (NewVelocity != GetVelocity()))
		{
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(RootComponent))
			{
				PrimitiveComponent->SetPhysicsLinearVelocity(NewVelocity);
			}
		}
	}
}

void AActor::PostNetReceivePhysicState()
{
	UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(RootComponent);
	if (RootPrimComp)
	{
		const FRepMovement& ThisReplicatedMovement = GetReplicatedMovement();
		FRigidBodyState NewState;
		ThisReplicatedMovement.CopyTo(NewState, this);
		RootPrimComp->SetRigidBodyReplicatedTarget(NewState, NAME_None, ThisReplicatedMovement.ServerFrame, ThisReplicatedMovement.ServerPhysicsHandle);
	}
}

void AActor::SetFakeNetPhysicsState(bool bShouldSleep)
{
	if (!IsNetMode(ENetMode::NM_Client))
	{
		return;
	}

	if (!IsReplicatingMovement())
	{
		return;
	}

	if (GetPhysicsReplicationMode() != EPhysicsReplicationMode::PredictiveInterpolation)
	{
		return;
	}

	UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(RootComponent);
	if (RootPrimComp)
	{
		FRigidBodyState RBState;
		RootPrimComp->GetRigidBodyState(RBState);

		RBState.Flags |= ERigidBodyFlags::NeedsUpdate;
		RBState.Flags |= ERigidBodyFlags::RepPhysics;

		if (bShouldSleep)
		{
			RBState.Flags |= ERigidBodyFlags::Sleeping;

			// Force no velocity for sleeping objects else it won't be able to go to sleep
			RBState.AngVel = FVector::ZeroVector;
			RBState.LinVel = FVector::ZeroVector;
		}

		RootPrimComp->SetRigidBodyReplicatedTarget(RBState, NAME_None, /*ServerFrame*/ INDEX_NONE, INDEX_NONE);
	}
}

void AActor::SyncReplicatedPhysicsSimulation()
{
	const FRepMovement& LocalRepMovement = GetReplicatedMovement();

	if (IsReplicatingMovement() && RootComponent && (RootComponent->IsSimulatingPhysics() != LocalRepMovement.bRepPhysics))
	{
		UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(RootComponent);
		if (RootPrimComp)
		{
			RootPrimComp->SetSimulatePhysics(LocalRepMovement.bRepPhysics);

			if(!LocalRepMovement.bRepPhysics)
			{
				if (UWorld* World = GetWorld())
				{
					if (FPhysScene* PhysScene = World->GetPhysicsScene())
					{
						if (IPhysicsReplication* PhysicsReplication = PhysScene->GetPhysicsReplication())
						{
							PhysicsReplication->RemoveReplicatedTarget(RootPrimComp);
						}
					}
				}
			}
		}
	}
}

bool AActor::IsWithinNetRelevancyDistance(const FVector& SrcLocation) const
{
	return FVector::DistSquared(SrcLocation, GetActorLocation()) < GetNetCullDistanceSquared();
}

bool AActor::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	if (bAlwaysRelevant || IsOwnedBy(ViewTarget) || IsOwnedBy(RealViewer) || this == ViewTarget || ViewTarget == GetInstigator())
	{
		return true;
	}
	else if (bNetUseOwnerRelevancy && Owner)
	{
		return Owner->IsNetRelevantFor(RealViewer, ViewTarget, SrcLocation);
	}
	else if (bOnlyRelevantToOwner)
	{
		return false;
	}
	else if (RootComponent && RootComponent->GetAttachParent() && RootComponent->GetAttachParent()->GetOwner() && (Cast<USkeletalMeshComponent>(RootComponent->GetAttachParent()) || (RootComponent->GetAttachParent()->GetOwner() == Owner)))
	{
		return RootComponent->GetAttachParent()->GetOwner()->IsNetRelevantFor(RealViewer, ViewTarget, SrcLocation);
	}
	else if(IsHidden() && (!RootComponent || !RootComponent->IsCollisionEnabled()))
	{
		return false;
	}

	if (!RootComponent)
	{
		UE_LOGF(LogNet, Warning, "Actor %ls / %ls has no root component in AActor::IsNetRelevantFor. (Make bAlwaysRelevant=true?)", *GetClass()->GetName(), *GetName() );
		return false;
	}

	return !GetDefault<AGameNetworkManager>()->bUseDistanceBasedRelevancy ||
			IsWithinNetRelevancyDistance(SrcLocation);
}

bool AActor::IsReplayRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation, const float CullDistanceOverrideSq) const
{
	return IsNetRelevantFor(RealViewer, ViewTarget, SrcLocation);
}

void AActor::GatherCurrentMovement()
{
	if (IsReplicatingMovement() || (RootComponent && RootComponent->GetAttachParent()))
	{
		bool bWasAttachmentModified = false;
		bool bWasRepMovementModified = false;

		AActor* OldAttachParent = AttachmentReplication.AttachParent;
		USceneComponent* OldAttachComponent = AttachmentReplication.AttachComponent;
	
		AttachmentReplication.AttachParent = nullptr;
		AttachmentReplication.AttachComponent = nullptr;

		UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(GetRootComponent());
		if (RootPrimComp && RootPrimComp->IsSimulatingPhysics())
		{
			const bool bPrevRepPhysics = ReplicatedMovement.bRepPhysics;

			const bool bShouldUsePhysicsReplicationCache = GetPhysicsReplicationMode() != EPhysicsReplicationMode::Default;
			bool bFoundInCache = false;

			UWorld* World = GetWorld();
			int ServerFrame = 0; 
			if (bShouldUsePhysicsReplicationCache && World && !World->IsRecordingClientReplay())
			{
				if (FPhysScene_Chaos* Scene = static_cast<FPhysScene_Chaos*>(World->GetPhysicsScene()))
				{
					if (const FRigidBodyState* FoundState = Scene->GetStateFromReplicationCache(RootPrimComp, /*OUT*/ServerFrame))
					{
						if (ReplicatedMovement.ServerFrame != ServerFrame)
						{
							ReplicatedMovement.FillFrom(*FoundState, this, ServerFrame);
							bWasRepMovementModified = true;
						}
						bFoundInCache = true;
					}
				}
			}

			if (!bFoundInCache)
			{
				// fallback to GT data
				FRigidBodyState RBState;
				RootPrimComp->GetRigidBodyState(RBState);
				ReplicatedMovement.FillFrom(RBState, this, ServerFrame);
				bWasRepMovementModified = true;
			}

			// Don't replicate movement if we're welded to another parent actor.
			// Their replication will affect our position indirectly since we are attached.
			ReplicatedMovement.bRepPhysics = !RootPrimComp->IsWelded();

			// If RepPhysics has changed value then notify the ReplicationSystem
			if (bPrevRepPhysics != ReplicatedMovement.bRepPhysics)
			{
				UpdateReplicatePhysicsCondition();
				bWasRepMovementModified = true;
			}

			if (GbSendPhysicsAttachRep)
			{
				if (RootComponent->GetAttachParent() != nullptr)
				{
					// Set the attachment info here to prevent it being detached. If the actor was previously not simulating physics
					// but now it is and is attached/welded, the attachment should remain
					AttachmentReplication.AttachParent = RootComponent->GetAttachParentActor();
					if (AttachmentReplication.AttachParent != nullptr)
					{
						AttachmentReplication.LocationOffset = RootComponent->GetRelativeLocation();
						AttachmentReplication.RotationOffset = RootComponent->GetRelativeRotation();
						AttachmentReplication.RelativeScale3D = RootComponent->GetRelativeScale3D();
						AttachmentReplication.AttachComponent = RootComponent->GetAttachParent();
						AttachmentReplication.AttachSocket = RootComponent->GetAttachSocketName();

						// Technically, the values might have stayed the same, but we'll just assume they've changed.
						bWasAttachmentModified = true;
					}
				}
			}
		}
		else if (RootComponent != nullptr)
		{
			// If we are attached, don't replicate absolute position, use AttachmentReplication instead.
			if (RootComponent->GetAttachParent() != nullptr)
			{
				// Networking for attachments assumes the RootComponent of the AttachParent actor. 
				// If that's not the case, we can't update this, as the client wouldn't be able to resolve the Component and would detach as a result.
				AttachmentReplication.AttachParent = RootComponent->GetAttachParentActor();
				if (AttachmentReplication.AttachParent != nullptr)
				{
					AttachmentReplication.LocationOffset = RootComponent->GetRelativeLocation();
					AttachmentReplication.RotationOffset = RootComponent->GetRelativeRotation();
					AttachmentReplication.RelativeScale3D = RootComponent->GetRelativeScale3D();
					AttachmentReplication.AttachComponent = RootComponent->GetAttachParent();
					AttachmentReplication.AttachSocket = RootComponent->GetAttachSocketName();

					// Technically, the values might have stayed the same, but we'll just assume they've changed.
					bWasAttachmentModified = true;
				}
			}
			else
			{
				ReplicatedMovement.Location = FRepMovement::RebaseOntoZeroOrigin(RootComponent->GetComponentLocation(), this);
				ReplicatedMovement.Rotation = RootComponent->GetComponentRotation();
				ReplicatedMovement.LinearVelocity = GetVelocity();
				ReplicatedMovement.AngularVelocity = FVector::ZeroVector;

				// Technically, the values might have stayed the same, but we'll just assume they've changed.
				bWasRepMovementModified = true;
			}

			bWasRepMovementModified = (bWasRepMovementModified || ReplicatedMovement.bRepPhysics);
			ReplicatedMovement.bRepPhysics = false;
		}

		if (bWasRepMovementModified)
		{
			MARK_PROPERTY_DIRTY_FROM_NAME(AActor, ReplicatedMovement, this);
		}

		if (bWasAttachmentModified ||
			OldAttachParent != AttachmentReplication.AttachParent ||
			OldAttachComponent != AttachmentReplication.AttachComponent)
		{
			MARK_PROPERTY_DIRTY_FROM_NAME(AActor, AttachmentReplication, this);
		}
	}
}

void AActor::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass());
	if (BPClass != NULL)
	{
		BPClass->GetLifetimeBlueprintReplicationList(OutLifetimeProps);
	}

	FDoRepLifetimeParams SharedParams;
	SharedParams.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, bReplicateMovement, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, Role, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, RemoteRole, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, Owner, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, bHidden, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, bTearOff, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, bCanBeDamaged, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, Instigator, SharedParams);

	constexpr bool bUsePushModel = true;
	
	FDoRepLifetimeParams AttachmentReplicationParams{COND_Custom, REPNOTIFY_Always, bUsePushModel};
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, AttachmentReplication, AttachmentReplicationParams);

	FDoRepLifetimeParams ReplicatedMovementParams{COND_SimulatedOrPhysics, REPNOTIFY_Always, bUsePushModel};
	DOREPLIFETIME_WITH_PARAMS_FAST(AActor, ReplicatedMovement, ReplicatedMovementParams);
}

void AActor::GetReplicatedCustomConditionState(FCustomPropertyConditionState& OutActiveState) const
{
	Super::GetReplicatedCustomConditionState(OutActiveState);

	DOREPCUSTOMCONDITION_ACTIVE_FAST(AActor, AttachmentReplication, RootComponent && !RootComponent->GetIsReplicated());
	DOREPCUSTOMCONDITION_ACTIVE_FAST(AActor, ReplicatedMovement, IsReplicatingMovement());
}

bool AActor::ReplicateSubobjects(UActorChannel *Channel, FOutBunch *Bunch, FReplicationFlags *RepFlags)
{
	check(Channel);
	check(Bunch);
	check(RepFlags);

	bool WroteSomething = false;

	for (UActorComponent* ActorComp : ReplicatedComponents)
	{
		if (ActorComp && ActorComp->GetReplicationCondition() != COND_Never)
		{
#if SUBOBJECT_TRANSITION_VALIDATION
			if (UActorChannel::CanIgnoreDeprecatedReplicateSubObjects() && !ActorComp->IsUsingRegisteredSubObjectList())
			{
				// Ignore components that only work with the old virtual method
				continue;
			}
#endif

			UActorChannel::SetCurrentSubObjectOwner(ActorComp);
			WroteSomething |= ActorComp->ReplicateSubobjects(Channel, Bunch, RepFlags);		// Lets the component add subobjects before replicating its own properties.

#if SUBOBJECT_TRANSITION_VALIDATION
			if (UActorChannel::CanIgnoreDeprecatedReplicateSubObjects())
			{
				// Don't replicate the component below when testing since we know this function is not voluntarily implemented
				continue;
			}
#endif

			UActorChannel::SetCurrentSubObjectOwner(this);
			WroteSomething |= Channel->ReplicateSubobject(ActorComp, *Bunch, *RepFlags);	// (this makes those subobjects 'supported', and from here on those objects may have reference replicated)		
		}
	}
	return WroteSomething;
}

ELifetimeCondition AActor::AllowActorComponentToReplicate(const UActorComponent* ComponentToReplicate) const
{
	return ComponentToReplicate->GetReplicationCondition();
}

void AActor::SetReplicatedComponentNetCondition(const UActorComponent* ReplicatedComponent, ELifetimeCondition NetCondition)
{
	checkf(HasActorBegunPlay(), TEXT("Can only set a netcondition after BeginPlay. %s setting for %s"), *GetName(), *ReplicatedComponent->GetName());

	if (FReplicatedComponentInfo* ComponentInfo = ReplicatedComponentsInfo.FindByKey(ReplicatedComponent))
	{
		if (NetCondition != ComponentInfo->NetCondition)
		{
			UE::Net::FReplicationSystemUtil::SetNetConditionForActorComponent(ReplicatedComponent, NetCondition);
			ComponentInfo->NetCondition = NetCondition;
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Tried to set a netcondition on a replicated component not part of the list yet, netcondition will be ignored. %s setting for %s"), *GetName(), *ReplicatedComponent->GetName());
	}
}

void AActor::AddComponentForReplication(UActorComponent* Component)
{
	AddComponentForReplication(Component, EAddComponentForReplicationBehavior::Immediate);
}

void AActor::AddComponentForReplication(UActorComponent* Component, EAddComponentForReplicationBehavior Type)
{
	constexpr EObjectFlags CDOFlags = RF_ClassDefaultObject | RF_ArchetypeObject;
	if (ActorHasBegunPlay == EActorBeginPlayState::HasNotBegunPlay || HasAnyFlags(CDOFlags))
	{
		return;
	}

	if (Component->bWantsInitializeComponent && !Component->HasBeenInitialized())
	{
		return;
	}

	const ELifetimeCondition NetCondition = AllowActorComponentToReplicate(Component);

	// Check if the UCS component was built from a template that's not replicated to the client.
	if (Component->CreationMethod == EComponentCreationMethod::UserConstructionScript && !Component->IsNameStableForNetworking())
	{
		ensureMsgf(Component->GetArchetype() == Component->GetClass()->GetDefaultObject(), TEXT("Replicated component %s::%s was added dynamically outside the construction script. This is not well supported and the component on the client will be initialized using the wrong archetype."),
			*GetName(), *Component->GetName());
	}

	ensureMsgf(ReplicatedComponents.Find(Component)!=INDEX_NONE, TEXT("AActor::AddComponentForReplication %s::%s but the componenbt was not found in the ReplicatedComponents list."), *GetName(), *Component->GetName());

	FReplicatedComponentInfo* ComponentInfo = ReplicatedComponentsInfo.FindByKey(Component);
	if (!ComponentInfo)
	{
		ReplicatedComponentsInfo.Emplace(FReplicatedComponentInfo(Component, NetCondition));
	}
	else
	{
		// Always set the condition because this component could be registering SubObjects ahead of time and already in the list with COND_Never
		ComponentInfo->NetCondition = NetCondition;
	}

	if (!Component->IsReadyForReplication())
	{
		Component->ReadyForReplication();
	}

    if (NetCondition != COND_Never)
    {
		if (Type == EAddComponentForReplicationBehavior::Immediate)
		{
			UE::Net::FReplicationSystemUtil::StartReplicatingActorComponent(this, Component);
		}
		else if (Type == EAddComponentForReplicationBehavior::Deferred)
		{
			UE::Net::FDeferredReplicationSystemCalls::StartReplicatingActorComponentDefer(this, Component);
		}
    }
}

void AActor::RemoveReplicatedComponent(UActorComponent* Component)
{
	int32 Index = ReplicatedComponentsInfo.IndexOfByKey(Component);
	if (Index != INDEX_NONE)
	{
		ReplicatedComponentsInfo.RemoveAtSwap(Index);
		if (HasAuthority())
		{
			UE::Net::FReplicationSystemUtil::StopReplicatingActorComponent(Component);
		}
	}
}

void AActor::AddReplicatedSubObject(UObject* SubObject, ELifetimeCondition NetCondition)
{
	if( !IsValid(SubObject) )
	{
		ensureMsgf(false, TEXT("Ignoring AddReplicatedSubObject for %s. Invalid pointer received."), *GetNameSafe(this));
		return;
	}

	if (SubObject->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		ensureMsgf(false, TEXT("AddReplicatedSubObject cannot replicate %s for %s. Archetypes or CDO's cannot be replicated subobject's."), *GetNameSafe(SubObject), *GetNameSafe(this));
		return;
	}

	ensureMsgf(IsUsingRegisteredSubObjectList(), TEXT("%s is registering subobjects but bReplicateUsingRegisteredSubObjectList is false. Without the flag set to true the registered subobjects will not be replicated."), *GetName());

	FSubObjectRegistry::EResult Result = ReplicatedSubObjects.AddSubObjectUnique(SubObject, NetCondition);

	UE_CLOGF(Result == FSubObjectRegistry::EResult::NewEntry, LogNetSubObject, Verbose, "%ls (0x%p) added replicated subobject %ls (0x%p) [%ls]", 
		*GetName(), this, *SubObject->GetName(), SubObject, *StaticEnum<ELifetimeCondition>()->GetValueAsString(NetCondition));

	// Warn if the subobject was registered with a different net condition.
	ensureMsgf(Result != FSubObjectRegistry::EResult::NetConditionConflict, TEXT("%s(0x%p) Registered subobject %s (0x%p) again with a different net condition. Active [%s] New [%s]. New condition will be ignored"),
		*GetName(), this, *SubObject->GetName(), SubObject, *StaticEnum<ELifetimeCondition>()->GetValueAsString(ReplicatedSubObjects.GetNetCondition(SubObject)), *StaticEnum<ELifetimeCondition>()->GetValueAsString(NetCondition));

	if (GetIsReplicated() && (ActorHasBegunPlay != EActorBeginPlayState::HasNotBegunPlay))
	{
		UE::Net::FReplicationSystemUtil::StartReplicatingActorSubObject(this, SubObject, NetCondition);
	}
}

bool AActor::RemoveReplicatedSubObjectFromList(UObject* SubObject)
{
	check(SubObject);
	const bool bWasRemoved = ReplicatedSubObjects.RemoveSubObject(SubObject);

	UE_CLOGF(bWasRemoved, LogNetSubObject, Verbose, "%ls (0x%p) removed replicated subobject %ls (0x%p)", *GetName(), this, *SubObject->GetName(), SubObject);
	return bWasRemoved;
}

void AActor::RemoveReplicatedSubObject(UObject* SubObject)
{
	const bool bWasRemoved = RemoveReplicatedSubObjectFromList(SubObject);
	if (bWasRemoved && HasAuthority())
	{
		UE::Net::FReplicationSystemUtil::StopReplicatingActorSubObject(this, SubObject);
	}
}

void AActor::DestroyReplicatedSubObjectOnRemotePeers(UObject* SubObject)
{
	check(SubObject);

	if (!HasAuthority())
	{
		// Only the authority can call this.
		return;
	}

	UE_LOGF(LogNetSubObject, Verbose, "%ls (0x%p) requested to Delete replicated subobject: %ls (0x%p)", *GetName(), this, *SubObject->GetName(), SubObject);

	if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(GetWorld()))
	{
		for (const FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver)
			{
				Driver.NetDriver->DeleteSubObjectOnClients(this, SubObject);
			}
		}
	}

	if (IsUsingRegisteredSubObjectList())
	{
		RemoveReplicatedSubObjectFromList(SubObject);
	}
}

void AActor::DestroyReplicatedSubObjectOnRemotePeers(UActorComponent* OwnerComponent, UObject* SubObject)
{
	check(SubObject);

	if (!HasAuthority())
	{
		// Only the authority can call this.
		return;
	}

	UE_LOGF(LogNetSubObject, Verbose, "%ls::%ls (0x%p) requested to Delete replicated subobject: %ls (0x%p)", *GetName(), *OwnerComponent->GetName(), OwnerComponent, *SubObject->GetName(), SubObject);

	if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(GetWorld()))
	{
		for (const FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver)
			{
				Driver.NetDriver->DeleteSubObjectOnClients(this, SubObject);
			}
		}
	}

	if (OwnerComponent->IsUsingRegisteredSubObjectList())
	{
		RemoveActorComponentReplicatedSubObjectFromList(OwnerComponent, SubObject);
	}
}

void AActor::TearOffReplicatedSubObjectOnRemotePeers(UActorComponent* OwnerComponent, UObject* SubObject)
{
	check(SubObject);

	if (!HasAuthority())
	{
		// Only the authority can call this.
		return;
	}

	UE_LOGF(LogNetSubObject, Verbose, "%ls::%ls (0x%p) requested to TearOff replicated subobject: %ls (0x%p)", *GetName(), *OwnerComponent->GetName(), this, *SubObject->GetName(), SubObject);

	if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(GetWorld()))
	{
		for (const FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver)
			{
				Driver.NetDriver->TearOffSubObjectOnClients(this, SubObject);
			}
		}
	}

	if (OwnerComponent->IsUsingRegisteredSubObjectList())
	{
		RemoveActorComponentReplicatedSubObjectFromList(OwnerComponent, SubObject);
	}
}

void AActor::TearOffReplicatedSubObjectOnRemotePeers(UObject* SubObject)
{
	check(SubObject);

	if (!HasAuthority())
	{
		// Only the authority can call this.
		return;
	}

	UE_LOGF(LogNetSubObject, Verbose, "%ls (0x%p) requested to TearOff replicated subobject: %ls (0x%p)", *GetName(), this, *SubObject->GetName(), SubObject);

	if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(GetWorld()))
	{
		for (const FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver)
			{
				Driver.NetDriver->TearOffSubObjectOnClients(this, SubObject);
			}
		}
	}

	if (IsUsingRegisteredSubObjectList())
	{
		RemoveReplicatedSubObjectFromList(SubObject);
	}

}

void AActor::AddActorComponentReplicatedSubObject(UActorComponent* OwnerComponent, UObject* SubObject, ELifetimeCondition NetCondition)
{
	check(IsValid(OwnerComponent));

	if (!IsValid(SubObject))
	{
		ensureMsgf(false, TEXT("Ignoring AddReplicatedSubObject for %s::%s. Invalid pointer received."), *GetNameSafe(this), *GetNameSafe(OwnerComponent));
		return;
	}

	if (SubObject->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		ensureMsgf(false, TEXT("AddReplicatedSubObject cannot replicate %s for %s::%s. Archetypes or CDO's cannot be replicated subobject's."), *GetNameSafe(SubObject), *GetNameSafe(this), *GetNameSafe(OwnerComponent));
		return;
	}
	
	ensureMsgf(OwnerComponent->GetIsReplicated(), TEXT("Only components with replication enabled can register subobjects. %s::%s has replication disabled."), *GetName(), *SubObject->GetName());
	ensureMsgf(NetCondition != COND_Custom, TEXT("Custom netconditions do not work with SubObjects. %s::%s will not be replicated."), *GetName(), *SubObject->GetName());

	FReplicatedComponentInfo* ComponentInfo = ReplicatedComponentsInfo.FindByKey(OwnerComponent);
	if (ComponentInfo)
	{
		// Add the subobject to the component's list
		FSubObjectRegistry::EResult Result = ComponentInfo->SubObjects.AddSubObjectUnique(SubObject, NetCondition);

		UE_CLOGF(Result == FSubObjectRegistry::EResult::NewEntry, LogNetSubObject, Verbose, "%ls::%ls (0x%p) added replicated subobject %ls (0x%p) [%ls]",
			*GetName(), *OwnerComponent->GetName(), OwnerComponent, *SubObject->GetName(), SubObject, *StaticEnum<ELifetimeCondition>()->GetValueAsString(NetCondition));

		// Warn if the subobject was registered with a different net condition.
		ensureMsgf(Result != FSubObjectRegistry::EResult::NetConditionConflict, TEXT("%s::%s (0x%p) Registered subobject %s (0x%p) again with a different net condition. Active [%s] New [%s]. New condition will be ignored"),
			*GetName(), *OwnerComponent->GetName(), OwnerComponent, *SubObject->GetName(), SubObject, *StaticEnum<ELifetimeCondition>()->GetValueAsString(ComponentInfo->SubObjects.GetNetCondition(SubObject)), *StaticEnum<ELifetimeCondition>()->GetValueAsString(NetCondition));

		if (OwnerComponent->GetIsReplicated() && OwnerComponent->HasBegunPlay())
		{
			UE::Net::FReplicationSystemUtil::StartReplicatingActorComponentSubObject(OwnerComponent, SubObject, NetCondition);
		}
	}
	else
	{
		// If we have to create an entry it means the component is registering subobjects while not replicating or before it's registered in the actor.
		// Let's allow it to register subobjects but set a condition that prevents it from getting replicated until we call IsAllowed.
		constexpr ELifetimeCondition NeverReplicate = COND_Never;

		const int32 Index = ReplicatedComponentsInfo.Emplace(FReplicatedComponentInfo(OwnerComponent, NeverReplicate));
		ReplicatedComponentsInfo[Index].SubObjects.AddSubObjectUnique(SubObject, NetCondition);

		UE_LOGF(LogNetSubObject, Verbose, "%ls::%ls (0x%p) added replicated subobject %ls (0x%p) [%ls]",
			*GetName(), *OwnerComponent->GetName(), OwnerComponent, *SubObject->GetName(), SubObject, *StaticEnum<ELifetimeCondition>()->GetValueAsString(NetCondition));
	}
}

bool AActor::RemoveActorComponentReplicatedSubObjectFromList(UActorComponent* OwnerComponent, UObject* SubObject)
{
	check(OwnerComponent);
	check(SubObject);

	if (FReplicatedComponentInfo* ComponentInfo = ReplicatedComponentsInfo.FindByKey(OwnerComponent))
	{
		const bool bWasRemoved = ComponentInfo->SubObjects.RemoveSubObject(SubObject);

		UE_CLOGF(bWasRemoved, LogNetSubObject, Verbose, "%ls::%ls (0x%p) removed replicated subobject %ls (0x%p)", *GetName(), *OwnerComponent->GetName(), OwnerComponent, *SubObject->GetName(), SubObject);

		return bWasRemoved;
	}

	return false;
}

void AActor::RemoveActorComponentReplicatedSubObject(UActorComponent* OwnerComponent, UObject* SubObject)
{
	const bool bWasRemoved = RemoveActorComponentReplicatedSubObjectFromList(OwnerComponent, SubObject);
	
	if (bWasRemoved && HasAuthority())
	{
		UE::Net::FReplicationSystemUtil::StopReplicatingActorComponentSubObject(OwnerComponent, SubObject);
	}
}

bool AActor::IsReplicatedSubObjectRegistered(const UObject* SubObject) const
{
	return ReplicatedSubObjects.IsSubObjectInRegistry(SubObject);
}

bool AActor::IsReplicatedActorComponentRegistered(const UActorComponent* ReplicatedComponent) const
{
	return ReplicatedComponentsInfo.FindByKey(ReplicatedComponent) != nullptr;
}

bool AActor::IsActorComponentReplicatedSubObjectRegistered(const UActorComponent* OwnerComponent, const UObject* SubObject) const
{
	check(OwnerComponent);

	if (const FReplicatedComponentInfo* ComponentInfo = ReplicatedComponentsInfo.FindByKey(OwnerComponent))
	{
		return ComponentInfo->SubObjects.IsSubObjectInRegistry(SubObject);
	}
	return false;
}

void AActor::BuildReplicatedComponentsInfo()
{
	checkf(HasActorBegunPlay() == false, TEXT("BuildReplicatedComponentsInfo can only be called before BeginPlay."));

	for (UActorComponent* ReplicatedComponent : ReplicatedComponents)
	{
		// Ask the actor if they want to override the replicated component
		const ELifetimeCondition NetCondition = AllowActorComponentToReplicate(ReplicatedComponent);

		const int32 Index = ReplicatedComponentsInfo.AddUnique(UE::Net::FReplicatedComponentInfo(ReplicatedComponent));
		ReplicatedComponentsInfo[Index].NetCondition = NetCondition;

		if (!ReplicatedComponent->IsReadyForReplication())
		{
			ReplicatedComponent->ReadyForReplication();
		}
	}
}

void AActor::OnSubobjectCreatedFromReplication(UObject *NewSubobject)
{
	check(NewSubobject);
	if ( UActorComponent * Component = Cast<UActorComponent>(NewSubobject) )
	{
		Component->OnCreatedFromReplication();
	}
	// Experimental
	else if (IInterface_ActorSubobject* SubojectInterface = Cast<IInterface_ActorSubobject>(NewSubobject))
	{
		SubojectInterface->OnCreatedFromReplication();
	}
}

/** Called on the actor when a subobject is dynamically destroyed via replication */
void AActor::OnSubobjectDestroyFromReplication(UObject *Subobject)
{
	check(Subobject);
	if ( UActorComponent * Component = Cast<UActorComponent>(Subobject) )
	{
		Component->OnDestroyedFromReplication();
	}
	// Experimental
	else if (IInterface_ActorSubobject* SubojectInterface = Cast<IInterface_ActorSubobject>(Subobject))
	{
		SubojectInterface->OnDestroyedFromReplication();
	}
}

void AActor::SetNetAddressable()
{
	bForceNetAddressable = 1;
}

bool AActor::IsNameStableForNetworking() const
{
	return IsNetStartupActor() || HasAnyFlags( RF_ClassDefaultObject | RF_ArchetypeObject ) || bForceNetAddressable != 0;
}

bool AActor::IsSupportedForNetworking() const
{
	return true;		// All actors are supported for networking
}

void AActor::OnRep_Owner()
{

}

void AActor::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	using namespace UE::Net;

	// Build descriptors and allocate PropertyReplicationFragments for this object
	FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);

	// $IRIS: $TODO: Consider splitting instance protocol / protocol creation, potentially protocol could be created from archetype/CDO or built as part of cook process.
	for (UActorComponent* ActorComp : OwnedComponents)
	{
		if (!IsValid(ActorComp))
		{
			continue;
		}

		if (ActorComp->HasAnyFlags(RF_DefaultSubObject) || ActorComp->IsDefaultSubobject())
		{
			if (!ActorComp->GetIsReplicated() || ActorComp->GetReplicationCondition() == COND_Never)
			{
				// Register RPC functions for not replicated default subobjects
				ActorComp->RegisterReplicationFragments(Context, EFragmentRegistrationFlags::RegisterRPCsOnly);
			}
		}
	}
}

void AActor::FillReplicationParams(const FFillReplicationParamsContext& Context, FActorReplicationParams& OutParams)
{
	OutParams = FActorReplicationParams();
}

void AActor::OnReplicationStartedForIris(const FOnReplicationStartedParams& Params)
{
	UpdateOwningNetConnection();

#if UE_WITH_REMOTE_OBJECT_HANDLE
	// Process any RPCs for this actor and its sub-objects (assumes the actor and its sub-objects have already been registered with Iris).
	UE::RemoteObject::Serialization::Network::ProcessRPCQueue(this);
#endif
}

void AActor::OnStopReplicationForIris(const FOnStopReplicationParams&)
{
}

void AActor::UpdateOwningNetConnection()
{
	using namespace UE::Net;

	UReplicationSystem* ReplicationSystem = FReplicationSystemUtil::GetReplicationSystem(this);
	UObjectReplicationBridge* ObjectReplicationBridge = (ReplicationSystem ? ReplicationSystem->GetReplicationBridge() : nullptr);
	if (ObjectReplicationBridge == nullptr)
	{
		return;
	}

	uint32 NewOwningNetConnectionId = 0U;
	if (const UNetConnection* NetConnection = GetNetConnection())
	{
		NewOwningNetConnectionId = NetConnection->GetConnectionHandle().GetParentConnectionId();
	}

	// If this actor isn't replicated there's no way for us to tell whether we need to update our children.
	bool bUpdateSelfAndChildren = !GetIsReplicated();
	if (!bUpdateSelfAndChildren)
	{
		const FNetHandle NetHandle = FReplicationSystemUtil::GetNetHandle(this);
		bUpdateSelfAndChildren = NetHandle.IsValid();
	}

	if (!bUpdateSelfAndChildren)
	{
		return;
	}

	TArray<AActor*, TInlineAllocator<32>> EveryChildren;
	// Include ourself
	EveryChildren.Push(this);

	do 
	{
		if (AActor* Actor = EveryChildren.Pop(EAllowShrinking::No))
		{
			for (TObjectPtr<AActor>& ActorChildPtr : Actor->Children)
			{
				if (ActorChildPtr.GetResidence() != EResidence::Remote)
				{
					AActor* ActorChild = ActorChildPtr;
					EveryChildren.Push(ActorChild);
				}
			}

			if (Actor->GetIsReplicated())
			{
				const UE::Net::FNetRefHandle RefHandle = ObjectReplicationBridge->GetReplicatedRefHandle(Actor);
				if (RefHandle.IsValid())
				{
					ReplicationSystem->SetOwningNetConnection(RefHandle, NewOwningNetConnectionId);
					// Update autonomous proxy condition
					const bool bEnableAutonomousCondition = Actor->GetRemoteRole() == ROLE_AutonomousProxy;
					ReplicationSystem->SetReplicationConditionConnectionFilter(RefHandle, EReplicationCondition::RoleAutonomous, NewOwningNetConnectionId, bEnableAutonomousCondition);
				}
			}
		}
	} while (EveryChildren.Num());
}


void AActor::UpdateReplicatePhysicsCondition()
{
	using namespace UE::Net;

	const FNetHandle NetHandle = FReplicationSystemUtil::GetNetHandle(this);
	if (NetHandle.IsValid())
	{
		FReplicationSystemUtil::SetReplicationCondition(NetHandle, EReplicationCondition::ReplicatePhysics, GetReplicatedMovement().bRepPhysics);
	}
}

bool AActor::CanTriggerResimulation() const
{
	return PhysicsReplicationMode == EPhysicsReplicationMode::Resimulation;
}

void AActor::SetPhysicsReplicationMode(const EPhysicsReplicationMode ReplicationMode)
{
	PhysicsReplicationMode = ReplicationMode;

	// When switching replication mode, also clear the current replication target.
	if (UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(RootComponent))
	{
		if (UWorld* World = GetWorld())
		{
			if (FPhysScene* PhysScene = World->GetPhysicsScene())
			{
				if (IPhysicsReplication* PhysicsReplication = PhysScene->GetPhysicsReplication())
				{
					PhysicsReplication->RemoveReplicatedTarget(RootPrimComp);
				}
			}
		}
	}
}

EPhysicsReplicationMode AActor::GetPhysicsReplicationMode()
{
	return PhysicsReplicationMode;
}


void AActor::RegisterAsFocalPointInPhysicsReplicationLOD() const
{
	if (UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(RootComponent))
	{
		RootPrimComp->RegisterAsFocalPointInPhysicsReplicationLOD();
	}
}

void AActor::UnregisterAsFocalPointInPhysicsReplicationLOD() const
{
	if (UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(RootComponent))
	{
		RootPrimComp->UnregisterAsFocalPointInPhysicsReplicationLOD();
	}
}

float AActor::GetResimulationThreshold() const
{
	return UPhysicsSettings::Get()->PhysicsPrediction.ResimulationSettings.ResimulationErrorPositionThreshold;
}
