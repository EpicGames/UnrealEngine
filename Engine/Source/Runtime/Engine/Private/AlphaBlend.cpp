// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlphaBlend.h"
#include "Curves/CurveFloat.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AlphaBlend)

FAlphaBlend::FAlphaBlend(float NewBlendTime) 
	: CustomCurve(nullptr)
	, BlendTime(FMath::Max(NewBlendTime, 0.0f))
	, BeginValue(0.0f)
	, DesiredValue(1.0f)
	, BlendOption(UE::Anim::DefaultBlendOption)
{
	Reset();
}

FAlphaBlend::FAlphaBlend(const FAlphaBlend& Other, float NewBlendTime)
	: CustomCurve(Other.CustomCurve)
	, BlendTime(FMath::Max(NewBlendTime, 0.0f))
	, BeginValue(Other.BeginValue)
	, DesiredValue(Other.DesiredValue)
	, BlendOption(Other.BlendOption)
{
	Reset();
}

#if WITH_EDITORONLY_DATA
bool FAlphaBlendArgs::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}

	return false;
}

bool FAlphaBlendArgs::Serialize(FStructuredArchive::FSlot Slot)
{
	FArchive& Ar = Slot.GetUnderlyingArchive();
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}
	return false;
}

void FAlphaBlendArgs::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::AlphaBlendNegativeBlendTimeGuard)
	{
		BlendTime = FMath::Max(BlendTime, 0.0f);
	}
}

bool FAlphaBlend::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}

	return false;
}

bool FAlphaBlend::Serialize(FStructuredArchive::FSlot Slot)
{
	FArchive& Ar = Slot.GetUnderlyingArchive();
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}
	return false;
}

void FAlphaBlend::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::AlphaBlendNegativeBlendTimeGuard)
	{
		BlendTime = FMath::Max(BlendTime, 0.0f);
	}
}

#endif // WITH_EDITORONLY_DATA

FAlphaBlend::FAlphaBlend(const FAlphaBlendArgs& InArgs)
	: CustomCurve(InArgs.CustomCurve)
	, BlendTime(FMath::Max(InArgs.BlendTime, 0.0f))
	, BeginValue(0.0f)
	, DesiredValue(1.0f)
	, BlendOption(InArgs.BlendOption)
{
	Reset();
}

void FAlphaBlend::ResetBlendTime()
{
	// if blend time is <= 0, then blending is done and complete
	if(BlendTime <= 0.f)
	{
		BlendTimeRemaining = 0.f;
		SetAlpha(1.f);
	}
	else
	{
		// Blend time is to go all the way, so scale that by how much we have to travel
		BlendTimeRemaining = BlendTime * FMath::Abs(1.f - AlphaLerp);
	}
	
	bNeedsToResetBlendTime = false;
}

void FAlphaBlend::ResetAlpha()
{
	float SmallerValue = FMath::Min(BeginValue, DesiredValue);
	float BiggerValue = FMath::Max(BeginValue, DesiredValue);
	// make sure it's within the range
	float NewBlendedValue = FMath::Clamp(BlendedValue, SmallerValue, BiggerValue);

	// if blend time is <= 0, or begin == end is same, there is nothing to be done
	// blending is done and complete
	if (BeginValue == DesiredValue)
	{
		SetAlpha(1.f);
	}
	else
	{
		AlphaLerp = (BlendedValue - BeginValue)/(DesiredValue - BeginValue);
		SetAlpha(AlphaLerp);
	}

	// reset the flag
	bNeedsToResetAlpha = false;
}

void FAlphaBlend::Reset()
{
	// Set alpha target to full - will also handle zero blend times
	// if blend time is zero, transition now, don't wait to call update.
	if( BlendTime <= 0.f )
	{
		SetAlpha(1.f);
		BlendTimeRemaining = 0.f;
	}
	else
	{
		SetAlpha(0.f);
		// Blend time is to go all the way, so scale that by how much we have to travel
		BlendTimeRemaining = BlendTime * FMath::Abs(1.f - AlphaLerp);
	}

	bNeedsToResetCachedDesiredBlendedValue = true;
	bNeedsToResetAlpha = false;
	bNeedsToResetBlendTime = false;
}

float FAlphaBlend::Update(float InDeltaTime)
{
	// Make sure passed in delta time is positive
	check(InDeltaTime >= 0.f);

	// check if we should reset alpha
	if (bNeedsToResetAlpha)
	{
		ResetAlpha();
	}

	// or should re calc blend time remaining
	if (bNeedsToResetBlendTime)
	{
		ResetBlendTime();
	}

	// if not complete, 
	if( !IsComplete() )
	{
		if( BlendTimeRemaining > InDeltaTime )
		{
			const float BlendDelta = 1.f - AlphaLerp; 
			AlphaLerp += (BlendDelta / BlendTimeRemaining) * InDeltaTime;
			BlendTimeRemaining -= InDeltaTime;
			SetAlpha(AlphaLerp);
		}
		else
		{
			// Cache our overshoot to report to caller
			float Overshoot = InDeltaTime - BlendTimeRemaining;

			BlendTimeRemaining = 0.f; 
			SetAlpha(1.f);

			return Overshoot;
		}
	}

	return 0.f;
}

float FAlphaBlend::AlphaToBlendOption()
{
	return AlphaToBlendOption(AlphaLerp, BlendOption, CustomCurve);
}

float FAlphaBlend::AlphaToBlendOption(float InAlpha, EAlphaBlendOption InBlendOption, UCurveFloat* InCustomCurve)
{
	switch(InBlendOption)
	{
		case EAlphaBlendOption::Sinusoidal:		return FMath::Clamp<float>((FMath::Sin(InAlpha * UE_PI - UE_HALF_PI) + 1.f) / 2.f, 0.f, 1.f);
		case EAlphaBlendOption::Cubic:			return FMath::Clamp<float>(FMath::CubicInterp<float>(0.f, 0.f, 1.f, 0.f, InAlpha), 0.f, 1.f);
		case EAlphaBlendOption::QuadraticInOut: return FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, InAlpha, 2), 0.f, 1.f);
		case EAlphaBlendOption::CubicInOut:		return FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, InAlpha, 3), 0.f, 1.f);
		case EAlphaBlendOption::HermiteCubic:	return FMath::Clamp<float>(FMath::SmoothStep(0.0f, 1.0f, InAlpha), 0.0f, 1.0f);
		case EAlphaBlendOption::QuarticInOut:	return FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, InAlpha, 4), 0.f, 1.f);
		case EAlphaBlendOption::QuinticInOut:	return FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, InAlpha, 5), 0.f, 1.f);
		case EAlphaBlendOption::CircularIn:		return FMath::Clamp<float>(FMath::InterpCircularIn<float>(0.0f, 1.0f, InAlpha), 0.0f, 1.0f);
		case EAlphaBlendOption::CircularOut:	return FMath::Clamp<float>(FMath::InterpCircularOut<float>(0.0f, 1.0f, InAlpha), 0.0f, 1.0f);
		case EAlphaBlendOption::CircularInOut:	return FMath::Clamp<float>(FMath::InterpCircularInOut<float>(0.0f, 1.0f, InAlpha), 0.0f, 1.0f);
		case EAlphaBlendOption::ExpIn:			return FMath::Clamp<float>(FMath::InterpExpoIn<float>(0.0f, 1.0f, InAlpha), 0.0f, 1.0f);
		case EAlphaBlendOption::ExpOut:			return FMath::Clamp<float>(FMath::InterpExpoOut<float>(0.0f, 1.0f, InAlpha), 0.0f, 1.0f);
		case EAlphaBlendOption::ExpInOut:		return FMath::Clamp<float>(FMath::InterpExpoInOut<float>(0.0f, 1.0f, InAlpha), 0.0f, 1.0f);
		case EAlphaBlendOption::Custom:
		{
			if(InCustomCurve)
			{
				float Min;
				float Max;
				InCustomCurve->GetTimeRange(Min, Max);
				return FMath::Clamp<float>(InCustomCurve->GetFloatValue(Min + (Max - Min) * InAlpha), 0.0f, 1.0f);
			}
		}
	}

	// Make sure linear returns a clamped value.
	return FMath::Clamp<float>(InAlpha, 0.f, 1.f);
}

void FAlphaBlend::SetValueRange(float Begin, float Desired)
{
	BeginValue = Begin;
	DesiredValue = Desired;

	bNeedsToResetAlpha = true;
	bNeedsToResetCachedDesiredBlendedValue = true;
}

/** Sets the final desired value for the blended value */
void FAlphaBlend::SetDesiredValue(float InDesired)
{
	SetValueRange(BlendedValue, InDesired);
}

/** note this function can modify BlendedValue right away */
void FAlphaBlend::SetAlpha(float InAlpha)
{
	AlphaLerp = FMath::Clamp(InAlpha, 0.0f, 1.0f);
	AlphaBlend = AlphaToBlendOption();
	BlendedValue = BeginValue + (DesiredValue - BeginValue) * AlphaBlend;
}

void FAlphaBlend::SetBlendTime(float InBlendTime)
{
	BlendTime = FMath::Max(InBlendTime, 0.f);
	// when blend time changes, we have to restart alpha
	bNeedsToResetBlendTime = true;
}

void FAlphaBlend::SetBlendOption(EAlphaBlendOption InBlendOption)
{
	BlendOption = InBlendOption;
	bNeedsToResetCachedDesiredBlendedValue = true;
}

void FAlphaBlend::SetCustomCurve(UCurveFloat* InCustomCurve)
{
	CustomCurve = InCustomCurve;
	bNeedsToResetCachedDesiredBlendedValue = true;
}

bool FAlphaBlend::IsComplete() const 
{
	if (bNeedsToResetCachedDesiredBlendedValue)
	{
		CachedDesiredBlendedValue = BeginValue + (DesiredValue - BeginValue) * AlphaToBlendOption(1.f, BlendOption, CustomCurve);
		bNeedsToResetCachedDesiredBlendedValue = false;
	}

	return (CachedDesiredBlendedValue == BlendedValue);
}

FAlphaBlendArgs::FAlphaBlendArgs()
	: CustomCurve(nullptr)
	, BlendTime(0.2f)
	, BlendOption(UE::Anim::DefaultBlendOption)
{

}

FAlphaBlendArgs::FAlphaBlendArgs(float InBlendTime)
	: CustomCurve(nullptr)
	, BlendTime(FMath::Max(InBlendTime, 0.f))
	, BlendOption(UE::Anim::DefaultBlendOption)
{

}

FAlphaBlendArgs::FAlphaBlendArgs(const struct FAlphaBlend& InAlphaBlend)
	: CustomCurve(InAlphaBlend.GetCustomCurve())
	, BlendTime(FMath::Max(InAlphaBlend.GetBlendTime(), 0.0f))
	, BlendOption(InAlphaBlend.GetBlendOption())
{

}

