// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Audio.cpp: Unreal base audio.
=============================================================================*/

#include "Sound/AmbientSound.h"
#include "Components/AudioComponent.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Sound/SoundBase.h"
#include "Misc/CommandLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AmbientSound)

#define LOCTEXT_NAMESPACE "AmbientSound"

/*-----------------------------------------------------------------------------
	AAmbientSound implementation.
-----------------------------------------------------------------------------*/
AAmbientSound::AAmbientSound(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AudioComponent = CreateDefaultSubobject<UAudioComponent>(TEXT("AudioComponent0"));

	AudioComponent->bAutoActivate = true;
	AudioComponent->bStopWhenOwnerDestroyed = true;
	AudioComponent->bShouldRemainActiveIfDropped = true;
	AudioComponent->Mobility = EComponentMobility::Movable;

	RootComponent = AudioComponent;

	bReplicates = false;
	SetHidden(true);
	SetCanBeDamaged(false);
}

#if WITH_EDITOR
void AAmbientSound::CheckForErrors( void )
{
	Super::CheckForErrors();

	if( !AudioComponent )
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT( "MapCheck_Message_AudioComponentNull", "Ambient sound actor has NULL AudioComponent property - please delete")))
			->AddToken(FMapErrorToken::Create(FMapErrors::AudioComponentNull));
	}
	else if( AudioComponent->Sound == NULL )
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT( "MapCheck_Message_SoundCueNull", "Ambient sound actor has NULL Sound Cue property")))
			->AddToken(FMapErrorToken::Create(FMapErrors::SoundCueNull));
	}
}

bool AAmbientSound::GetReferencedContentObjects( TArray<UObject*>& Objects ) const
{
	Super::GetReferencedContentObjects(Objects);

	if (AudioComponent->Sound)
	{
		Objects.Add( AudioComponent->Sound );
	}
	return true;
}

#endif

void AAmbientSound::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

#if WITH_EDITORONLY_DATA
	if ( AudioComponent && bHiddenEdLevel )
	{
		AudioComponent->Stop();
	}
#endif // WITH_EDITORONLY_DATA
}

FString AAmbientSound::GetInternalSoundCueName()
{
	FString CueName;
#if WITH_EDITOR
	CueName = GetActorLabel();
#endif
	if (CueName.Len() == 0)
	{
		CueName = GetName();
	}
	CueName += TEXT("_SoundCue");

	return CueName;
}

void AAmbientSound::FadeIn(float FadeInDuration, float FadeVolumeLevel)
{
	if (AudioComponent)
	{
		AudioComponent->FadeIn(FadeInDuration, FadeVolumeLevel);
	}
}

void AAmbientSound::FadeOut(float FadeOutDuration, float FadeVolumeLevel)
{
	if (AudioComponent)
	{
		AudioComponent->FadeOut(FadeOutDuration, FadeVolumeLevel);
	}
}

void AAmbientSound::AdjustVolume(float AdjustVolumeDuration, float AdjustVolumeLevel)
{
	if (AudioComponent)
	{
		AudioComponent->AdjustVolume(AdjustVolumeDuration, AdjustVolumeLevel);
	}
}

void AAmbientSound::Play(float StartTime)
{
	if (AudioComponent)
	{
		static bool bMuteAmbientActors = FParse::Param(FCommandLine::Get(), TEXT("NoAmbientActors"));
		if (!bMuteAmbientActors)
		{
			AudioComponent->Play(StartTime);
		}
	}
}

void AAmbientSound::Stop()
{
	if (AudioComponent)
	{
		AudioComponent->Stop();
	}
}

#undef LOCTEXT_NAMESPACE


