// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorAnalyticsSessionSummary.h"

#if WITH_EDITOR

#include "Editor.h"
#include "IVREditorModule.h"
#include "Framework/Application/SlateApplication.h"

// includes for editor focus summary test
#include "Framework/Application/SlateUser.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Widgets/Docking/SDockTab.h"

#include "EditorUsageAnalyticsSummary.h"

#include "Interfaces/IAnalyticsPropertyStore.h"
#include "Kismet2/DebuggerCommands.h"
#include "ProfilingDebugging/StallDetector.h"
#include "ShaderCompiler.h"


namespace EditorAnalyticsProperties
{
	// List of mutable keys.
	static const TAnalyticsProperty<uint32>  UserInteractionCount   = TEXT("UserInteractionCount");
	static const TAnalyticsProperty<int32>   Idle1Min               = TEXT("1MinIdle");
	static const TAnalyticsProperty<int32>   Idle5Min               = TEXT("5MinIdle");
	static const TAnalyticsProperty<int32>   Idle30Min              = TEXT("30MinIdle");
	static const TAnalyticsProperty<bool>    IsInPIE                = TEXT("IsInPIE");
	static const TAnalyticsProperty<bool>    IsInVRMode             = TEXT("IsInVRMode");
	static const TAnalyticsProperty<bool>    IsUnattended           = TEXT("IsUnattended");
	static const TAnalyticsProperty<uint32>  DirtyPackageCount      = TEXT("DirtyPackageCount");

	static const TAnalyticsProperty<uint32>  TotalStallCount        = TEXT("TotalStallCount");
	static const TAnalyticsProperty<uint32>  TotalStallReported     = TEXT("TotalStallReported");
	static const TAnalyticsProperty<uint32>  TopStallTriggerCount   = TEXT("TopStallTriggerCount");
	static const TAnalyticsProperty<FString> TopStallName           = TEXT("TopStallName");
	static const TAnalyticsProperty<double>  TopStallBudgetSeconds  = TEXT("TopStallBudgetSeconds");
	static const TAnalyticsProperty<double>  TopStallOverageSeconds = TEXT("TopStallOverageSeconds");
	static const TAnalyticsProperty<uint32>  ProcessDiagnostics     = TEXT("ProcessDiagnostics"); // Whether some profiling/diagnostic tools are enabled, which could slow down the Editor.
	static const TAnalyticsProperty<uint32>  SummaryEventVersion    = TEXT("SummaryEventVersion"); // A version number used identify the key/set used. Can be used to compare before/after some changes too since comparing engine version is not always straingth foward.

	static const TAnalyticsProperty<uint32> ShadersCompiled			= TEXT("ShadersCompiled");
	static const TAnalyticsProperty<uint32> ShaderDDCMisses			= TEXT("ShaderDDCMisses");
	static const TAnalyticsProperty<uint32> ShaderDDCHits			= TEXT("ShaderDDCHits");
	static const TAnalyticsProperty<double> TimeShaderCompilationWasActive = TEXT("TimeShaderCompilationWasActive ");
}

namespace EditorAnalyticsCVars
{
	static TAutoConsoleVariable<bool> CVarDisableFocusTimes(
		TEXT("EditorAnalytics.DisableFocusTimes"),
		false,
		TEXT("Disables collecting focused widget times for analytics summary."));
}


FEditorAnalyticsSessionSummary::FEditorAnalyticsSessionSummary(TSharedPtr<IAnalyticsPropertyStore> Store, uint32 MonitorProcessId)
	: FEngineAnalyticsSessionSummary(MoveTemp(Store), MonitorProcessId)
	, LastUserActivityTimeSecs(FPlatformTime::Seconds())
	, AccountedUserIdleSecs(0)
{
	EditorAnalyticsProperties::UserInteractionCount.Set(GetStore(), 0);
	EditorAnalyticsProperties::Idle1Min.Set(GetStore(), 0);
	EditorAnalyticsProperties::Idle5Min.Set(GetStore(), 0);
	EditorAnalyticsProperties::Idle30Min.Set(GetStore(), 0);
	EditorAnalyticsProperties::IsInVRMode.Set(GetStore(), IVREditorModule::Get().GetVRModeBase() != nullptr);
	EditorAnalyticsProperties::IsInPIE.Set(GetStore(), FPlayWorldCommandCallbacks::IsInPIE());
	EditorAnalyticsProperties::IsUnattended.Set(GetStore(), FApp::IsUnattended());
	EditorAnalyticsProperties::DirtyPackageCount.Set(GetStore(), 0);

	EditorAnalyticsProperties::TopStallName.Set(GetStore(), TEXT(""), /*Capacity*/128);
	EditorAnalyticsProperties::TopStallBudgetSeconds.Set(GetStore(), 0.0);
	EditorAnalyticsProperties::TopStallOverageSeconds.Set(GetStore(), 0.0);
	EditorAnalyticsProperties::TopStallTriggerCount.Set(GetStore(), 0);
	EditorAnalyticsProperties::TotalStallCount.Set(GetStore(), 0);
	EditorAnalyticsProperties::TotalStallReported.Set(GetStore(), 0);
	EditorAnalyticsProperties::ProcessDiagnostics.Set(GetStore(), static_cast<uint32>(FPlatformMisc::GetProcessDiagnostics()));

	// The current summary revision number. Identify the key set used and its behavior. If we add/remove keys, we should increment the number. The number can
	// also be incremented when we change some behaviors to be able to compare between versions. Mostly useful for 'dev' branches where users bypasses UGS to update/build.
	//    - V3 -> Windows optimization for stall/ensure -> The engine only captures the responsible thread so CRC walks 1 thread rather than all threads.
	//    - V4 -> Added DirtyPackageCount property.
	//    - V5 -> Added CPUInfo to the base class (EngineAnalyticsSessionSummary)
	//    - V6 -> Added shader compilation statistics.
	EditorAnalyticsProperties::SummaryEventVersion.Set(GetStore(), 6);

	EditorAnalyticsProperties::ShadersCompiled.Set(GetStore(), 0);
	EditorAnalyticsProperties::ShaderDDCMisses.Set(GetStore(), 0);
	EditorAnalyticsProperties::ShaderDDCHits.Set(GetStore(), 0);
	EditorAnalyticsProperties::TimeShaderCompilationWasActive.Set(GetStore(), 0.0);

	// Persist the session to disk.
	GetStore()->Flush();

	FEditorDelegates::PreBeginPIE.AddRaw(this, &FEditorAnalyticsSessionSummary::OnEnterPIE);
	FEditorDelegates::EndPIE.AddRaw(this, &FEditorAnalyticsSessionSummary::OnExitPIE);
	FSlateApplication::Get().GetOnModalLoopTickEvent().AddRaw(this, &FEditorAnalyticsSessionSummary::Tick);
	FSlateApplication::Get().GetLastUserInteractionTimeUpdateEvent().AddRaw(this, &FEditorAnalyticsSessionSummary::OnSlateUserInteraction);
	UPackage::PackageDirtyStateChangedEvent.AddRaw(this, &FEditorAnalyticsSessionSummary::OnDirtyPackageStateChanged);

	if (!EditorAnalyticsCVars::CVarDisableFocusTimes.GetValueOnAnyThread())
	{
		// we want to make sure an instance is created so the usage summary starts listening to events
		FEditorUsageAnalyticsSummary::Get().Start();
	}
}

void FEditorAnalyticsSessionSummary::ShutdownInternal()
{
	if (!EditorAnalyticsCVars::CVarDisableFocusTimes.GetValueOnAnyThread())
	{
		FEditorUsageAnalyticsSummary::Get().Shutdown();
	}

	FEditorDelegates::PreBeginPIE.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);
	FSlateApplication::Get().GetOnModalLoopTickEvent().RemoveAll(this);
	FSlateApplication::Get().GetLastUserInteractionTimeUpdateEvent().RemoveAll(this);
	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
}

bool FEditorAnalyticsSessionSummary::UpdateSessionProgressInternal(bool bCrashing)
{
	bool bShouldPersist = UpdateUserIdleTime(FPlatformTime::Seconds(), /*bReset*/false);

	// In case of crash, don't save anything else.
	if (bCrashing)
	{
		return true; // Yes, should persist.
	}

	EditorAnalyticsProperties::IsInVRMode.Set(GetStore(), IVREditorModule::Get().GetVRModeBase() != nullptr);
	EditorAnalyticsProperties::IsInPIE.Set(GetStore(), FPlayWorldCommandCallbacks::IsInPIE());

	// Accumulate stall stats
#if STALL_DETECTOR
	TArray<UE::FStallDetectorStats::TabulatedResult> StallResults;
	UE::FStallDetectorStats::TabulateStats(StallResults);
	if (!StallResults.IsEmpty())
	{
		UE::FStallDetectorStats::TabulatedResult TopResult(StallResults[0]);
		EditorAnalyticsProperties::TopStallName.Set(GetStore(), TopResult.Name, /*Capacity*/ 128);
		EditorAnalyticsProperties::TopStallBudgetSeconds.Set(GetStore(), TopResult.BudgetSeconds);
		EditorAnalyticsProperties::TopStallOverageSeconds.Set(GetStore(), TopResult.OverageSeconds);
		EditorAnalyticsProperties::TopStallTriggerCount.Set(GetStore(), TopResult.TriggerCount);
	}

	EditorAnalyticsProperties::TotalStallCount.Set(GetStore(), UE::FStallDetectorStats::TotalTriggeredCount.Get());
	EditorAnalyticsProperties::TotalStallReported.Set(GetStore(), UE::FStallDetectorStats::TotalReportedCount.Get());
#endif // STALL_DETECTOR

	// Shader stats.
	EditorAnalyticsProperties::ShadersCompiled.Set(GetStore(), GShaderCompilerStats->GetTotalShadersCompiled());
	EditorAnalyticsProperties::ShaderDDCMisses.Set(GetStore(), GShaderCompilerStats->GetDDCMisses());
	EditorAnalyticsProperties::ShaderDDCHits.Set(GetStore(), GShaderCompilerStats->GetDDCHits());
	EditorAnalyticsProperties::TimeShaderCompilationWasActive.Set(GetStore(), GShaderCompilerStats->GetTimeShaderCompilationWasActive());

	return bShouldPersist;
}

bool FEditorAnalyticsSessionSummary::UpdateUserIdleTime(double CurrTimeSecs, bool bReset)
{
	bool bSessionUpdated = false;

	// How much time elapsed since the last activity.
	double TotalIdleSecs = CurrTimeSecs - LastUserActivityTimeSecs.load();
	if (TotalIdleSecs > 60.0) // Less than a minute is always considered normal interaction delay.
	{
		double LastAccountedIdleSecs = AccountedUserIdleSecs.load();
		double UnaccountedIdleSecs   = TotalIdleSecs - LastAccountedIdleSecs;

		// If one or more minute is unaccounted
		if (UnaccountedIdleSecs >= 60.0)
		{
			double AccountedIdleMins = FMath::FloorToDouble(LastAccountedIdleSecs/60); // Minutes already accounted for.
			double ToAccountIdleMins = FMath::FloorToDouble(UnaccountedIdleSecs/60);   // New minutes to account for (entire minute only)

			// Delta = LatestAccounted - AlreadyAccounted
			double DeltaIdle1Min  = FMath::Max(0.0, AccountedIdleMins + ToAccountIdleMins - 1)  - FMath::Max(0.0, AccountedIdleMins - 1);  // The first minute of this idle sequence is considered 'normal interaction delay' and in not accounted as idle.
			double DeltaIdle5Min  = FMath::Max(0.0, AccountedIdleMins + ToAccountIdleMins - 5)  - FMath::Max(0.0, AccountedIdleMins - 5);  // The 5 first minutes of this idle sequence are considered 'normal interaction delay' and are not accounted for the 5-min timer.
			double DeltaIdle30Min = FMath::Max(0.0, AccountedIdleMins + ToAccountIdleMins - 30) - FMath::Max(0.0, AccountedIdleMins - 30); // The 30 first minutes of this idle sequence are considered 'normal interaction delay' and are not accounted for the 30-min timer.

			// Ensure only one thread adds the current delta time.
			if (AccountedUserIdleSecs.compare_exchange_strong(LastAccountedIdleSecs, LastAccountedIdleSecs + ToAccountIdleMins * 60.0)) // Only add the 'accounted' minutes and keep fraction of minutes running.
			{
				EditorAnalyticsProperties::Idle1Min.Update(GetStore(), [DeltaIdle1Min](int32& InOutValue)
				{
					InOutValue += FMath::RoundToInt(static_cast<float>(DeltaIdle1Min));
					return true;
				});

				EditorAnalyticsProperties::Idle5Min.Update(GetStore(), [DeltaIdle5Min](int32& InOutValue)
				{
					InOutValue += FMath::RoundToInt(static_cast<float>(DeltaIdle5Min));
					return true;
				});

				EditorAnalyticsProperties::Idle30Min.Update(GetStore(), [DeltaIdle30Min](int32& InOutValue)
				{
					InOutValue += FMath::RoundToInt(static_cast<float>(DeltaIdle30Min));
					return true;
				});

				bSessionUpdated = true;
			}
		}
	}

	if (bReset)
	{
		AccountedUserIdleSecs.store(0);
		LastUserActivityTimeSecs.store(CurrTimeSecs);
	}

	return bSessionUpdated; // True if the idle timers were updated.
}

void FEditorAnalyticsSessionSummary::OnSlateUserInteraction(double CurrSlateInteractionTime)
{
	EditorAnalyticsProperties::UserInteractionCount.Update(GetStore(), [](uint32& InOutValue)
	{
		++InOutValue;
		return true;
	});

	// If the user input 'reset' the idle timers.
	double CurrTimeSecs = FPlatformTime::Seconds();
	if (UpdateUserIdleTime(CurrTimeSecs, /*bReset*/true))
	{
		GetStore()->Flush(/*bAsync*/true, FTimespan::Zero());
	}
}

void FEditorAnalyticsSessionSummary::OnEnterPIE(const bool /*bIsSimulating*/)
{
	EditorAnalyticsProperties::IsInPIE.Set(GetStore(), true);
	GetStore()->Flush(/*bAsync*/true, FTimespan::Zero());
}

void FEditorAnalyticsSessionSummary::OnExitPIE(const bool /*bIsSimulating*/)
{
	EditorAnalyticsProperties::IsInPIE.Set(GetStore(), false);
	GetStore()->Flush(/*bAsync*/true, FTimespan::Zero());
}

void FEditorAnalyticsSessionSummary::OnDirtyPackageStateChanged(UPackage* InPackage)
{
	if (InPackage->HasAnyPackageFlags(PKG_CompiledIn))
	{
		return; // We are interested to know if user has unsaved work. Compiled in package are irrelevant for that.
	}
	else if (InPackage->IsDirty()) // Newly created + modified packages
	{
		EditorAnalyticsProperties::DirtyPackageCount.Update(GetStore(), [](uint32& Actual)
		{
			++Actual;
			return true;
		});
	}
	else
	{
		EditorAnalyticsProperties::DirtyPackageCount.Update(GetStore(), [](uint32& Actual)
		{
			if (Actual > 0) // In case the callback was register too late and missed some changes.
			{
				--Actual;
				return true;
			}
			return false;
		});
	}

	GetStore()->Flush(/*bAsync*/true, FTimespan::Zero());
}

#endif // WITH_EDITOR
