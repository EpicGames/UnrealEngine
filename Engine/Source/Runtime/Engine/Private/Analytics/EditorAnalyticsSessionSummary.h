// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR

#include "Analytics/EngineAnalyticsSessionSummary.h"
#include <atomic>

class UPackage;

/** Specializes the base engine analytics for the Editor, adding Editor only data to the summary. */
class FEditorAnalyticsSessionSummary : public FEngineAnalyticsSessionSummary
{
public:
	FEditorAnalyticsSessionSummary(TSharedPtr<IAnalyticsPropertyStore> Store, uint32 MonitorProcessId);

protected:
	/** Invoked by base class FEngineAnalyticsSessionSummary at the configured update period. */
	virtual bool UpdateSessionProgressInternal(bool bCrashing) override;

	/** Invoked by base class FEngineAnalyticsSessionSummary on shutdown. */
	virtual void ShutdownInternal() override;

private:
	bool UpdateUserIdleTime(double CurrTimeSecs, bool bReset);
	void OnSlateUserInteraction(double CurrSlateInteractionTime);
	void OnEnterPIE(const bool /*bIsSimulating*/);
	void OnExitPIE(const bool /*bIsSimulating*/);
	void OnDirtyPackageStateChanged(UPackage* package);

private:
	/** Last activity (user input, crash, terminate, shutdown) timestamp from FPlatformTime::Seconds() to track user inactivity. */
	std::atomic<double> LastUserActivityTimeSecs;

	/** The number of idle seconds in the current idle sequence that were accounted (saved in the session) for the user idle counters. */
	std::atomic<double> AccountedUserIdleSecs;
};

#endif // WITH_EDITOR
