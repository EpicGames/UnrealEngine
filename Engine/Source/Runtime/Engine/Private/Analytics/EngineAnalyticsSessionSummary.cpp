// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analytics/EngineAnalyticsSessionSummary.h"
#include "AnalyticsSessionSummaryManager.h"
#include "Engine/EngineTypes.h"
#include "GeneralProjectSettings.h"
#include "Engine/Engine.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Interfaces/IAnalyticsPropertyStore.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Misc/EngineVersion.h"
#include "UserActivityTracking.h"
#include "RHI.h"
#include "RHIStats.h"

#if PLATFORM_WINDOWS
#include "HAL/FileManager.h"
#endif

extern ENGINE_API float GAverageFPS;
extern CORE_API bool GIsGPUCrashed;

namespace EngineAnalyticsProperties
{
	// List of mutable properties. These need to be in a singleton to prevent undefined static initialization ordering
	class FProperties
	{
	public:
		static const FProperties& Get() { static FProperties Instance; return Instance; }

		const TAnalyticsProperty<FDateTime> Timestamp;
		const TAnalyticsProperty<FDateTime> LastTickTimestamp;
		const TAnalyticsProperty<int32> SessionDurationSecs;
		const TAnalyticsProperty<uint64> EngineTickCount;
		const TAnalyticsProperty<uint64> SessionTickCount;
		const TAnalyticsProperty<FString> UserActivity;
		const TAnalyticsProperty<int32>  MonitorExitCode;
		const TAnalyticsProperty<float> AverageFPS;
		const TAnalyticsProperty<bool> IsTerminating;
		const TAnalyticsProperty<bool> IsHang;
		const TAnalyticsProperty<bool> WasShutdown;
		const TAnalyticsProperty<bool> IsCrashing;
		const TAnalyticsProperty<bool> IsGpuCrash;
		const TAnalyticsProperty<bool> IsVanilla;
		const TAnalyticsProperty<bool> IsLowDriveSpace;
		const TAnalyticsProperty<bool> WasDebuggerIgnored;
		const TAnalyticsProperty<bool> WasEverDebugged;
		const TAnalyticsProperty<bool> IsCrcMissing;
		const TAnalyticsProperty<bool> RunningOnBattery;
		const TAnalyticsProperty<bool> IsUserLoggingOut;
		const TAnalyticsProperty<int32> ShutdownType;

	private:
		FProperties()
			: Timestamp(TEXT("Timestamp"))
			, LastTickTimestamp(TEXT("LastTickTimestamp"))
			, SessionDurationSecs(TEXT("SessionDuration"))
			, EngineTickCount(TEXT("EngineTickCount"))
			, SessionTickCount(TEXT("SessionTickCount"))
			, UserActivity(TEXT("CurrentUserActivity"))
			, MonitorExitCode(TEXT("MonitorExitCode"))
			, AverageFPS(TEXT("AverageFPS"))
			, IsTerminating(TEXT("IsTerminating"))
			, IsHang(TEXT("IsHang"))
			, WasShutdown(TEXT("WasShutdown"))
			, IsCrashing(TEXT("IsCrash"))
			, IsGpuCrash(TEXT("GPUCrash"))
			, IsVanilla(TEXT("IsVanilla"))
			, IsLowDriveSpace(TEXT("IsLowDriveSpace"))
			, WasDebuggerIgnored(TEXT("WasDebuggerIgnored"))
			, WasEverDebugged(TEXT("WasDebugged"))
			, IsCrcMissing(TEXT("IsCrcMissing"))
			, RunningOnBattery(TEXT("RunningOnBattery"))
			, IsUserLoggingOut(FAnalyticsSessionSummaryManager::IsUserLoggingOutProperty)
			, ShutdownType(FAnalyticsSessionSummaryManager::ShutdownTypeCodeProperty)
		{}
	};

	static FString UnknownUserActivity("Unknown");

	float GetAverageFPS()
	{
		return GAverageFPS;
	}

	uint64 GetEngineTickCount()
	{
		return GFrameCounter;
	}

	bool IsCrcExecutableMissing()
	{
		// TODO: Add a function later on to PlatfomCrashContext to check existance of CRC. This is only used on windows at the moment to categorize cases where CRC fails to report the exit code (only supported on Windows).
	#if PLATFORM_WINDOWS
		const FString EngineDir = FPlatformMisc::EngineDir();

		// Find the path to crash reporter binary. Avoid creating FStrings.
		FString CrcPathRel = EngineDir / TEXT("Binaries") / FPlatformProcess::GetBinariesSubdirectory() / TEXT("CrashReportClientEditor.exe");
		FString CrcPathDev = EngineDir / TEXT("Binaries") / FPlatformProcess::GetBinariesSubdirectory() / TEXT("CrashReportClientEditor-Win64-Development.exe");
		FString CrcPathDbg = EngineDir / TEXT("Binaries") / FPlatformProcess::GetBinariesSubdirectory() / TEXT("CrashReportClientEditor-Win64-Debug.exe");

		return !IFileManager::Get().FileExists(*CrcPathRel) && !IFileManager::Get().FileExists(*CrcPathDev) && !IFileManager::Get().FileExists(*CrcPathDbg);
	#else
		return false;
	#endif
	}

	FString GetProjectName(const UGeneralProjectSettings& ProjectSettings)
	{
		FString ProjectName = FApp::GetProjectName();
		if (ProjectName.Len() && ProjectSettings.ProjectName.Len())
		{
			if (ProjectSettings.ProjectName != ProjectName)
			{
				ProjectName = ProjectName / ProjectSettings.ProjectName; // The project names don't match, report both.
			}
		}
		else if (ProjectName.IsEmpty())
		{
			ProjectName = ProjectSettings.ProjectName;
		}

		return ProjectName;
	}

	FString GetUserActivityString(const FUserActivity& InUserActivity)
	{
		return InUserActivity.ActionName.IsEmpty() ? UnknownUserActivity : InUserActivity.ActionName;
	}

	bool ShouldOverwriteShutdownType(const int32* ActualShutdownType, const int32& ProposedShutdownType)
	{
		if (static_cast<EAnalyticsSessionShutdownType>(ProposedShutdownType) == EAnalyticsSessionShutdownType::Crashed) // Crash overwrite everything else.
		{
			return true;
		}
		else if (static_cast<EAnalyticsSessionShutdownType>(*ActualShutdownType) == EAnalyticsSessionShutdownType::Crashed) // Cannot overwrite a crash.
		{
			return false;
		}
		else if (static_cast<EAnalyticsSessionShutdownType>(ProposedShutdownType) == EAnalyticsSessionShutdownType::Debugged) // Debugger can overwrite Shutdown|Terminated|Abnormal|Hang
		{
			return true;
		}
		else if (static_cast<EAnalyticsSessionShutdownType>(*ActualShutdownType) == EAnalyticsSessionShutdownType::Debugged) // Shutdown|Terminated|Abnormal|Hang cannot overwrite the debugger.
		{
			return false;
		}
		return true; // Shutdown|Terminate|Abnormal|Hang can overwrite each other.
	}
} // namespace EngineAnalyticsProperties


FEngineAnalyticsSessionSummary::FEngineAnalyticsSessionSummary(TSharedPtr<IAnalyticsPropertyStore> InStore, uint32 InMonitorProcessId)
	: Store(MoveTemp(InStore))
	, SessionStartTimeUtc(FDateTime::UtcNow())
	, SessionStartTimeSecs(FPlatformTime::Seconds())
	, CrcProcessId(InMonitorProcessId)
	, PersistPeriod(FTimespan::FromSeconds(10))
{
	// List the plugins.
	TArray<TSharedRef<IPlugin>> Plugins = IPluginManager::Get().GetEnabledPlugins();
	TArray<FString> PluginNames;
	PluginNames.Reserve(Plugins.Num());
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		PluginNames.Add(Plugin->GetName());
	}
	PluginNames.Sort();

	// Get the operating system information.
	FString OSMajor;
	FString OSMinor;
	FPlatformMisc::GetOSVersions(/*out*/ OSMajor, /*out*/ OSMinor);

	FTextureMemoryStats TextureMemStats;
	RHIGetTextureMemoryStats(TextureMemStats);

	// Get project settings.
	const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();

	// Create the immutable metrics.
	Store->Set(TEXT("Platform"), FString(FPlatformProperties::IniPlatformName()));
#if PLATFORM_MAC
#if PLATFORM_MAC_ARM64
    Store->Set(TEXT("UEBuildArch"), FString("AppleSilicon"));
#else
    Store->Set(TEXT("UEBuildArch"), FString("Intel(Mac)"));
#endif
#endif
	Store->Set(TEXT("OSMajor"), OSMajor);
	Store->Set(TEXT("OSMinor"), OSMinor);
	Store->Set(TEXT("OSVersion"), FPlatformMisc::GetOSVersion());
	Store->Set(TEXT("Is64BitOS"), true);
	Store->Set(TEXT("TotalPhysicalRAM"), FPlatformMemory::GetStats().TotalPhysical);
	Store->Set(TEXT("CPUPhysicalCores"), FPlatformMisc::NumberOfCores());
	Store->Set(TEXT("CPULogicalCores"), FPlatformMisc::NumberOfCoresIncludingHyperthreads());
	Store->Set(TEXT("CPUVendor"), FPlatformMisc::GetCPUVendor());
	Store->Set(TEXT("CPUBrand"), FPlatformMisc::GetCPUBrand());
	Store->Set(TEXT("CPUInfo"), FPlatformMisc::GetCPUInfo()); // This is a bitfield. See the function documentation in GenericPlatformMisc.h to interpret the bits.
	Store->Set(TEXT("DesktopGPUAdapter"), FPlatformMisc::GetPrimaryGPUBrand());
	Store->Set(TEXT("RenderingGPUAdapter"), GRHIAdapterName);
	Store->Set(TEXT("GPUVendorID"), GRHIVendorId);
	Store->Set(TEXT("GPUDeviceID"), GRHIDeviceId);
	Store->Set(TEXT("GRHIDeviceRevision"), GRHIDeviceRevision);
	Store->Set(TEXT("GRHIAdapterInternalDriverVersion"), GRHIAdapterInternalDriverVersion);
	Store->Set(TEXT("GRHIAdapterUserDriverVersion"), GRHIAdapterUserDriverVersion);
	Store->Set(TEXT("GRHIName"), FString(GDynamicRHI ? GDynamicRHI->GetName() : TEXT("")));
	Store->Set(TEXT("GRHIAdapterDriverOnDenyList"), GRHIAdapterDriverOnDenyList);
	Store->Set(TEXT("GRHIAdapterMemory"), static_cast<uint64>(TextureMemStats.DedicatedVideoMemory));
	Store->Set(TEXT("ProjectName"), EngineAnalyticsProperties::GetProjectName(ProjectSettings));
	Store->Set(TEXT("ProjectID"), ProjectSettings.ProjectID.ToString(EGuidFormats::DigitsWithHyphens));
	Store->Set(TEXT("ProjectDescription"), ProjectSettings.Description);
	Store->Set(TEXT("ProjectVersion"), ProjectSettings.ProjectVersion);
	Store->Set(TEXT("EngineVersion"), FEngineVersion::Current().ToString(EVersionComponent::Changelist));
	Store->Set(TEXT("CommandLine"), FString(FCommandLine::GetForLogging()));
	Store->Set(TEXT("MonitorPid"), CrcProcessId); // CrashReportClient acts as the monitoring/reporting process.
	Store->Set(TEXT("Plugins"), FString::Join(PluginNames, TEXT(",")));
	Store->Set(TEXT("IsCrcMissing"), CrcProcessId != 0 ? false : EngineAnalyticsProperties::IsCrcExecutableMissing()); // CrashReportClient is the monitor process. If we have a process id, it's not missing.
	Store->Set(TEXT("IsInEnterprise"), IProjectManager::Get().IsEnterpriseProject());
	Store->Set(TEXT("StartupTimestamp"), SessionStartTimeUtc);

	FDateTime CurrTimeUtc = FDateTime::UtcNow();

	// Create the mutable metrics.
	EngineAnalyticsProperties::FProperties::Get().Timestamp.Set(GetStore(), CurrTimeUtc);
	EngineAnalyticsProperties::FProperties::Get().LastTickTimestamp.Set(GetStore(), CurrTimeUtc);
	EngineAnalyticsProperties::FProperties::Get().SessionDurationSecs.Set(GetStore(), FPlatformTime::Seconds() - SessionStartTimeSecs);
	EngineAnalyticsProperties::FProperties::Get().EngineTickCount.Set(GetStore(), EngineAnalyticsProperties::GetEngineTickCount());
	EngineAnalyticsProperties::FProperties::Get().SessionTickCount.Set(GetStore(), CurrSessionTickCount);
	EngineAnalyticsProperties::FProperties::Get().UserActivity.Set(GetStore(), EngineAnalyticsProperties::GetUserActivityString(FUserActivityTracking::GetUserActivity()));
	EngineAnalyticsProperties::FProperties::Get().AverageFPS.Set(GetStore(), EngineAnalyticsProperties::GetAverageFPS());
	EngineAnalyticsProperties::FProperties::Get().IsTerminating.Set(GetStore(),  false);
	EngineAnalyticsProperties::FProperties::Get().IsHang.Set(GetStore(), false);
	EngineAnalyticsProperties::FProperties::Get().WasShutdown.Set(GetStore(), false);
	EngineAnalyticsProperties::FProperties::Get().IsCrashing.Set(GetStore(), false);
	EngineAnalyticsProperties::FProperties::Get().IsGpuCrash.Set(GetStore(), false);
	EngineAnalyticsProperties::FProperties::Get().IsUserLoggingOut.Set(GetStore(), false);
	EngineAnalyticsProperties::FProperties::Get().IsVanilla.Set(GetStore(), GEngine && GEngine->IsVanillaProduct());
	EngineAnalyticsProperties::FProperties::Get().IsLowDriveSpace.Set(GetStore(), false);
	EngineAnalyticsProperties::FProperties::Get().WasDebuggerIgnored.Set(GetStore(), false);
	EngineAnalyticsProperties::FProperties::Get().WasEverDebugged.Set(GetStore(), false);
	EngineAnalyticsProperties::FProperties::Get().ShutdownType.Set(GetStore(), (int32)EAnalyticsSessionShutdownType::Abnormal);
	EngineAnalyticsProperties::FProperties::Get().RunningOnBattery.Set(GetStore(), FPlatformMisc::IsRunningOnBattery());

	Store->Flush();

	// Listen to interesting events.
	FCoreDelegates::OnHandleSystemError.AddRaw(this, &FEngineAnalyticsSessionSummary::OnCrashing); // WARNING: Don't assume this function is only called from game thread.
	FCoreDelegates::OnHandleSystemHang.AddRaw(this, &FEngineAnalyticsSessionSummary::OnHung); // WARNING: Don't assume this function is only called from game thread.
	FCoreDelegates::GetApplicationWillTerminateDelegate().AddRaw(this, &FEngineAnalyticsSessionSummary::OnTerminate); // WARNING: Don't assume this function is only called from game thread.
	FCoreDelegates::IsVanillaProductChanged.AddRaw(this, &FEngineAnalyticsSessionSummary::OnVanillaStateChanged);
	FCoreDelegates::OnUserLoginChangedEvent.AddRaw(this, &FEngineAnalyticsSessionSummary::OnUserLoginChanged);
	FUserActivityTracking::OnActivityChanged.AddRaw(this, &FEngineAnalyticsSessionSummary::OnUserActivity);
}

void FEngineAnalyticsSessionSummary::Shutdown()
{
	if (!bShutdown)
	{
		// Call the derived class.
		ShutdownInternal();

		FCoreDelegates::OnHandleSystemError.RemoveAll(this);
		FCoreDelegates::OnHandleSystemHang.RemoveAll(this);
		FCoreDelegates::GetApplicationWillTerminateDelegate().RemoveAll(this);
		FCoreDelegates::IsVanillaProductChanged.RemoveAll(this);
		FCoreDelegates::OnUserLoginChangedEvent.RemoveAll(this);
		FUserActivityTracking::OnActivityChanged.RemoveAll(this);

		EngineAnalyticsProperties::FProperties::Get().WasShutdown.Set(GetStore(), true);
		EngineAnalyticsProperties::FProperties::Get().ShutdownType.Set(GetStore(), (int32)EAnalyticsSessionShutdownType::Shutdown, EngineAnalyticsProperties::ShouldOverwriteShutdownType);
		UpdateSessionProgress(/*bFlushAsync*/false, /*FlushTimeout*/FTimespan::MaxValue()); // Forces a sunchronous flush.
		bShutdown = true;
	}
}

void FEngineAnalyticsSessionSummary::Tick(float DeltaTime)
{
	if (bShutdown)
	{
		return;
	}

	// The number of times this instance was ticked.
	EngineAnalyticsProperties::FProperties::Get().SessionTickCount.Update(GetStore(), [](uint64& Value) { ++Value; return true; });

	// The average FPS.
	EngineAnalyticsProperties::FProperties::Get().AverageFPS.Set(GetStore(), EngineAnalyticsProperties::GetAverageFPS());

	// This tick timestamp. Not allowed to go back.
	EngineAnalyticsProperties::FProperties::Get().LastTickTimestamp.Set(GetStore(), FDateTime::UtcNow(), [](const FDateTime* Actual, const FDateTime& Proposed){ return Proposed >*Actual; });

	// Detect if CRC state changed since the last update.
	bool bFlush = UpdateExternalProcessReporterState(/*bQuickCheck*/true);

	// Check if the debugger states changed.
	bFlush |= UpdateDebuggerStates();

	// Ensure to flush the properties periodically.
	if (bFlush || FPlatformTime::Seconds() >= NextPersistTimeSeconds)
	{
		UpdateSessionProgress();
		NextPersistTimeSeconds = FPlatformTime::Seconds() + PersistPeriod.GetTotalSeconds();
	}
}

void FEngineAnalyticsSessionSummary::OnCrashing()
{
	EngineAnalyticsProperties::FProperties::Get().IsCrashing.Set(GetStore(), true);
	EngineAnalyticsProperties::FProperties::Get().IsGpuCrash.Set(GetStore(), GIsGPUCrashed);
	EngineAnalyticsProperties::FProperties::Get().ShutdownType.Set(GetStore(), (int32)EAnalyticsSessionShutdownType::Crashed);

	// Flush the store if possible but don't wait forever to avoid deadlocking if the crash affects the store ability to complete its background task.
	UpdateSessionProgress(/*bFlushAsync*/false, /*FlushTimeout*/FTimespan::FromMilliseconds(100), /*bCrashing*/true);
}

void FEngineAnalyticsSessionSummary::OnHung()
{
	EngineAnalyticsProperties::FProperties::Get().IsHang.Set(GetStore(), true);
	EngineAnalyticsProperties::FProperties::Get().ShutdownType.Set(GetStore(), (int32)EAnalyticsSessionShutdownType::Hang, EngineAnalyticsProperties::ShouldOverwriteShutdownType);
	UpdateSessionProgress(/*bFlushAsync*/false, /*FlushTimeout*/FTimespan::FromMilliseconds(100));
}

void FEngineAnalyticsSessionSummary::OnTerminate()
{
	EngineAnalyticsProperties::FProperties::Get().IsTerminating.Set(GetStore(), true);
	EngineAnalyticsProperties::FProperties::Get().ShutdownType.Set(GetStore(), (int32)EAnalyticsSessionShutdownType::Terminated, EngineAnalyticsProperties::ShouldOverwriteShutdownType);
	UpdateSessionProgress(/*bFlushAsync*/false, /*FlushTimeout*/FTimespan::FromMilliseconds(100));
}

void FEngineAnalyticsSessionSummary::OnUserLoginChanged(bool bLoggingIn, int32, int32)
{
	if (!bLoggingIn)
	{
		EngineAnalyticsProperties::FProperties::Get().IsUserLoggingOut.Set(GetStore(), true);
		UpdateSessionProgress(/*bFlushAsync*/false, /*FlushTimeout*/FTimespan::FromMilliseconds(100));
	}
}

void FEngineAnalyticsSessionSummary::OnVanillaStateChanged(bool bIsVanilla)
{
	EngineAnalyticsProperties::FProperties::Get().IsVanilla.Set(GetStore(), bIsVanilla);
	UpdateSessionProgress();
}

void FEngineAnalyticsSessionSummary::LowDriveSpaceDetected()
{
	EngineAnalyticsProperties::FProperties::Get().IsLowDriveSpace.Set(GetStore(), true);
	UpdateSessionProgress();
}

void FEngineAnalyticsSessionSummary::OnUserActivity(const FUserActivity& InUserActivity)
{
	EngineAnalyticsProperties::FProperties::Get().UserActivity.Set(GetStore(), EngineAnalyticsProperties::GetUserActivityString(InUserActivity));
	UpdateSessionProgress();
}

void FEngineAnalyticsSessionSummary::UpdateSessionProgress(bool bFlushAsync, const FTimespan& FlushTimeout, bool bCrashing)
{
	// Let the derived classes update.
	UpdateSessionProgressInternal(bCrashing);

	// Current session duration in seconds.
	EngineAnalyticsProperties::FProperties::Get().SessionDurationSecs.Set(GetStore(), FMath::FloorToInt(static_cast<float>(FPlatformTime::Seconds() - SessionStartTimeSecs)),
		[](const int32* Actual, const int32& Proposed) { return Proposed > *Actual; });

	// Wall timestamp.
	EngineAnalyticsProperties::FProperties::Get().Timestamp.Set(GetStore(), FDateTime::UtcNow(),
		[](const FDateTime* Actual, const FDateTime& Proposed) { return Proposed > *Actual; });

	// The number of times the engine ticked.
	EngineAnalyticsProperties::FProperties::Get().EngineTickCount.Set(GetStore(), EngineAnalyticsProperties::GetEngineTickCount(),
		[](const uint64* Actual, const uint64& Proposed) { return Proposed > *Actual; });

	// Flush (or try to flush) the store.
	Store->Flush(bFlushAsync, FlushTimeout);
}

bool FEngineAnalyticsSessionSummary::UpdateExternalProcessReporterState(bool bQuickCheck)
{
	if (CrcProcessId == 0)
	{
		return false; // Nothing to update, monitor is not running in background (not supported/not in monitor mode/failed to launch)
	}
	else if (CrcExitCode.IsSet() && *CrcExitCode != ECrashExitCodes::OutOfProcessReporterExitedUnexpectedly)
	{
		return false; // Already have the real exit code set.
	}
	else if (TOptional<int32> ExitCode = FGenericCrashContext::GetOutOfProcessCrashReporterExitCode())
	{
		CrcExitCode = MoveTemp(ExitCode);
		EngineAnalyticsProperties::FProperties::Get().MonitorExitCode.Set(GetStore(), *CrcExitCode);
		UpdateSessionProgress();
		return true;
	}
	else if (bQuickCheck)
	{
		return false; // All the code above is pretty fast and can run every tick. IsApplicationRunning() is slow, so exit here.
	}
	else if (!CrcExitCode.IsSet() && !FPlatformProcess::IsApplicationRunning(CrcProcessId))
	{
		// Set a rather unique, but known exit code as place holder, hoping that next update, the engine will report the real one.
		CrcExitCode.Emplace(ECrashExitCodes::OutOfProcessReporterExitedUnexpectedly);
		EngineAnalyticsProperties::FProperties::Get().MonitorExitCode.Set(GetStore(), *CrcExitCode);
		UpdateSessionProgress();
		return true;
	}
	// else -> either CrashReportClientEditor is still running or we already flagged it as dead.
	return false;
}

bool FEngineAnalyticsSessionSummary::UpdateDebuggerStates()
{
	bool bStateChanged = false;
	bool bIgnoreDebugger = false;

#if !UE_BUILD_SHIPPING
	bIgnoreDebugger = GIgnoreDebugger;
#endif // !UE_BUILD_SHIPPING

	// Ignoring the debugger changes how IsDebuggerPresent() behave and masks the usage of the debugger if true.
	if (!bDebuggerIgnored && bIgnoreDebugger)
	{
		bStateChanged = true;
		bDebuggerIgnored = true;
		bWasEverDebugged = true;
		EngineAnalyticsProperties::FProperties::Get().WasDebuggerIgnored.Set(GetStore(), true);
		EngineAnalyticsProperties::FProperties::Get().WasEverDebugged.Set(GetStore(), true); // Assumes the session was debugged if the debugger was ignored.
		EngineAnalyticsProperties::FProperties::Get().ShutdownType.Set(GetStore(), (int32)EAnalyticsSessionShutdownType::Debugged, EngineAnalyticsProperties::ShouldOverwriteShutdownType);
	}

	// Check if the debugger is attached.
	if (!bWasEverDebugged && FPlatformMisc::IsDebuggerPresent())
	{
		bStateChanged = true;
		bWasEverDebugged = true;
		EngineAnalyticsProperties::FProperties::Get().WasEverDebugged.Set(GetStore(), true);
		EngineAnalyticsProperties::FProperties::Get().ShutdownType.Set(GetStore(), (int32)EAnalyticsSessionShutdownType::Debugged, EngineAnalyticsProperties::ShouldOverwriteShutdownType);
	}

	return bStateChanged;
}
