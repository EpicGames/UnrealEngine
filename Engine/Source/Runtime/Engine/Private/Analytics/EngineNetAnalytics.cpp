// Copyright Epic Games, Inc. All Rights Reserved.


// Includes
#include "Analytics/EngineNetAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineLogs.h"
#include "Serialization/JsonWriter.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Net/Core/Connection/NetCloseResult.h"


/**
 * FNetConnAnalyticsVars
 */

FNetConnAnalyticsVars::FNetConnAnalyticsVars()
	: OutAckOnlyCount(0)
	, OutKeepAliveCount(0)
{
}

bool FNetConnAnalyticsVars::operator == (const FNetConnAnalyticsVars& A) const
{
	return OutAckOnlyCount == A.OutAckOnlyCount &&
			OutKeepAliveCount == A.OutKeepAliveCount &&
			OutOfOrderPacketsLostCount == A.OutOfOrderPacketsLostCount &&
			OutOfOrderPacketsRecoveredCount == A.OutOfOrderPacketsRecoveredCount &&
			OutOfOrderPacketsDuplicateCount == A.OutOfOrderPacketsDuplicateCount &&
			/** Close results can't be shared - if either are set, equality comparison fails */
			!CloseReason.IsValid() && !A.CloseReason.IsValid() &&
			ClientCloseReasons == A.ClientCloseReasons &&
			RecoveredFaults.OrderIndependentCompareEqual(A.RecoveredFaults) &&
			FailedPingAddressesICMP == A.FailedPingAddressesICMP &&
			FailedPingAddressesUDP == A.FailedPingAddressesUDP;
}

void FNetConnAnalyticsVars::CommitAnalytics(FNetConnAnalyticsVars& AggregatedData)
{
	AggregatedData.OutAckOnlyCount += OutAckOnlyCount;
	AggregatedData.OutKeepAliveCount += OutKeepAliveCount;
	AggregatedData.OutOfOrderPacketsLostCount += OutOfOrderPacketsLostCount;
	AggregatedData.OutOfOrderPacketsRecoveredCount += OutOfOrderPacketsRecoveredCount;
	AggregatedData.OutOfOrderPacketsDuplicateCount += OutOfOrderPacketsDuplicateCount;

	for (TMap<FString, int32>::TConstIterator It(RecoveredFaults); It; ++It)
	{
		int32& CountValue = AggregatedData.RecoveredFaults.FindOrAdd(It.Key());

		CountValue += It.Value();
	}

	FPerNetConnData& CurData = AggregatedData.PerConnectionData.AddDefaulted_GetRef();

	if (CloseReason.IsValid())
	{
		CurData.CloseReason = MoveTemp(CloseReason);
		CurData.ClientCloseReasons = MoveTemp(ClientCloseReasons);
	}

	AggregatedData.FailedPingAddressesICMP.Append(FailedPingAddressesICMP);
	AggregatedData.FailedPingAddressesUDP.Append(FailedPingAddressesUDP);
}


/**
 * FNetConnAnalyticsData
 */

void FNetConnAnalyticsData::SendAnalytics()
{
	using namespace UE::Net;

	FNetConnAnalyticsVars NullVars;
	const TSharedPtr<IAnalyticsProvider>& AnalyticsProvider = Aggregator->GetAnalyticsProvider();

	if (!(*this == NullVars) && AnalyticsProvider.IsValid())
	{
		struct FReasonCounter
		{
			FString				ReasonStr;
			int32				Counter		= 0;
		};

		TArray<FReasonCounter> HeadCloseReasons;
		TArray<FReasonCounter> FullCloseReasons;
		TArray<FReasonCounter> HeadClientCloseReasons;
		TArray<FReasonCounter> FullClientCloseReasons;

		auto FindOrAddReason =
			[](TArray<FReasonCounter>& InArray, FString InReasonStr) -> int32&
			{
				for (FReasonCounter& CurEntry : InArray)
				{
					if (CurEntry.ReasonStr == InReasonStr)
					{
						return CurEntry.Counter;
					}
				}

				FReasonCounter& NewReason = InArray.AddDefaulted_GetRef();

				NewReason.ReasonStr = InReasonStr;

				return NewReason.Counter;
			};

		for (const FPerNetConnData& CurData : PerConnectionData)
		{
			if (!CurData.CloseReason.IsValid())
			{
				int32& CurHeadVal = FindOrAddReason(HeadCloseReasons, LexToString(ENetCloseResult::Unknown));
				int32& CurFullVal = FindOrAddReason(FullCloseReasons, LexToString(ENetCloseResult::Unknown));

				CurHeadVal++;
				CurFullVal++;
			}
			else
			{
				int32& CurHeadVal = FindOrAddReason(HeadCloseReasons, CurData.CloseReason->DynamicToString(ENetResultString::ResultEnumOnly));

				CurHeadVal++;

				FString CurFullReason;

				for (FNetResult::FConstIterator It(*CurData.CloseReason); It; ++It)
				{
					FString CurReason = It->DynamicToString(ENetResultString::ResultEnumOnly);
					TStringBuilder<256> CurFormattedReason;

					if (!CurFullReason.IsEmpty())
					{
						CurFormattedReason.AppendChar(TEXT(','));
					}

					CurFormattedReason.Append(ToCStr(CurReason));

					CurFullReason += CurFormattedReason.ToString();
				}

				int32& CurFullVal = FindOrAddReason(FullCloseReasons, CurFullReason);

				CurFullVal++;
			}

			if (CurData.ClientCloseReasons.Num() > 0)
			{
				bool bFirstVal = true;
				FString CurFullReason;

				for (const FString& CurClientReason : CurData.ClientCloseReasons)
				{
					if (bFirstVal)
					{
						int32& CurHeadClientVal = FindOrAddReason(HeadClientCloseReasons, CurClientReason);

						CurHeadClientVal++;
					}

					bFirstVal = false;

					TStringBuilder<256> CurFormattedReason;

					if (!CurFullReason.IsEmpty())
					{
						CurFormattedReason.AppendChar(TEXT(','));
					}

					CurFormattedReason.Append(ToCStr(CurClientReason));

					CurFullReason += CurFormattedReason.ToString();
				}

				int32& CurFullVal = FindOrAddReason(FullClientCloseReasons, CurFullReason);

				CurFullVal++;
			}
		}

		{
			auto CounterSort = [](const FReasonCounter& A, const FReasonCounter& B) -> bool
				{
					return A.Counter < B.Counter;
				};

			HeadCloseReasons.Sort(CounterSort);
			FullCloseReasons.Sort(CounterSort);
			HeadClientCloseReasons.Sort(CounterSort);
			FullClientCloseReasons.Sort(CounterSort);
		}


		struct FFailedPingCounter
		{
			FString PingAddress;
			int32 Counter = 0;

			bool operator == (const FString& A) const
			{
				return PingAddress == A;
			}
		};

		TArray<FFailedPingCounter> ProcessedFailedPingAddressesICMP;
		TArray<FFailedPingCounter> ProcessedFailedPingAddressesUDP;

		for (const FString& CurFailedAddress : FailedPingAddressesICMP)
		{
			int32 ProcessedIdx = ProcessedFailedPingAddressesICMP.IndexOfByKey(CurFailedAddress);

			if (ProcessedIdx == INDEX_NONE)
			{
				ProcessedIdx = ProcessedFailedPingAddressesICMP.Add({CurFailedAddress, 0});
			}

			ProcessedFailedPingAddressesICMP[ProcessedIdx].Counter++;
		}

		for (const FString& CurFailedAddress : FailedPingAddressesUDP)
		{
			int32 ProcessedIdx = ProcessedFailedPingAddressesUDP.IndexOfByKey(CurFailedAddress);

			if (ProcessedIdx == INDEX_NONE)
			{
				ProcessedIdx = ProcessedFailedPingAddressesUDP.Add({CurFailedAddress, 0});
			}

			ProcessedFailedPingAddressesUDP[ProcessedIdx].Counter++;
		}

		{
			auto CounterSort = [](const FFailedPingCounter& A, const FFailedPingCounter& B) -> bool
				{
					return A.Counter < B.Counter;
				};

			ProcessedFailedPingAddressesICMP.Sort(CounterSort);
			ProcessedFailedPingAddressesUDP.Sort(CounterSort);
		}


		UE_LOGF(LogNet, Log, "NetConnection Analytics:");

		UE_LOGF(LogNet, Log, " - OutAckOnlyCount: %llu", OutAckOnlyCount);
		UE_LOGF(LogNet, Log, " - OutKeepAliveCount: %llu", OutKeepAliveCount);
		UE_LOGF(LogNet, Log, " - OutOfOrderPacketsLostCount: %llu", OutOfOrderPacketsLostCount);
		UE_LOGF(LogNet, Log, " - OutOfOrderPacketsRecoveredCount: %llu", OutOfOrderPacketsRecoveredCount);
		UE_LOGF(LogNet, Log, " - OutOfOrderPacketsDuplicateCount: %llu", OutOfOrderPacketsDuplicateCount);


		if (HeadCloseReasons.Num() > 0)
		{
			UE_LOGF(LogNet, Log, " - CloseReasons:");

			for (const FReasonCounter& CurCounter : HeadCloseReasons)
			{
				UE_LOGF(LogNet, Log, "  - %ls: %i", ToCStr(CurCounter.ReasonStr), CurCounter.Counter);
			}
		}

		if (FullCloseReasons.Num() > 0)
		{
			UE_LOGF(LogNet, Log, " - FullCloseReasons:");

			for (const FReasonCounter& CurCounter : FullCloseReasons)
			{
				UE_LOGF(LogNet, Log, "  - %ls: %i", ToCStr(CurCounter.ReasonStr), CurCounter.Counter);
			}
		}


		if (HeadClientCloseReasons.Num() > 0)
		{
			UE_LOGF(LogNet, Log, " - ClientCloseReasons:");

			for (const FReasonCounter& CurCounter : HeadClientCloseReasons)
			{
				UE_LOGF(LogNet, Log, "  - %ls: %i", ToCStr(CurCounter.ReasonStr), CurCounter.Counter);
			}
		}

		if (FullClientCloseReasons.Num() > 0)
		{
			UE_LOGF(LogNet, Log, " - FullClientCloseReasons:");

			for (const FReasonCounter& CurCounter : FullClientCloseReasons)
			{
				UE_LOGF(LogNet, Log, "  - %ls: %i", ToCStr(CurCounter.ReasonStr), CurCounter.Counter);
			}
		}

		if (RecoveredFaults.Num() > 0)
		{
			UE_LOGF(LogNet, Log, " - RecoveredFaults:");

			for (TMap<FString, int32>::TConstIterator It(RecoveredFaults); It; ++It)
			{
				UE_LOGF(LogNet, Log, "  - %ls: %i", ToCStr(It.Key()), It.Value());
			}
		}

		if (ProcessedFailedPingAddressesICMP.Num() > 0)
		{
			UE_LOGF(LogNet, Log, " - FailedPingAddressesICMP:");

			for (const FFailedPingCounter& CurCounter : ProcessedFailedPingAddressesICMP)
			{
				UE_LOGF(LogNet, Log, "  - %ls: %i", ToCStr(CurCounter.PingAddress), CurCounter.Counter);
			}
		}

		if (ProcessedFailedPingAddressesUDP.Num() > 0)
		{
			UE_LOGF(LogNet, Log, " - FailedPingAddressesUDP:");

			for (const FFailedPingCounter& CurCounter : ProcessedFailedPingAddressesUDP)
			{
				UE_LOGF(LogNet, Log, "  - %ls: %i", ToCStr(CurCounter.PingAddress), CurCounter.Counter);
			}
		}


		static const FString EZEventName = "Core.ServerNetConn";
		static const FString EZAttrib_OutAckOnlyCount = "OutAckOnlyCount";
		static const FString EZAttrib_OutKeepAliveCount = "OutKeepAliveCount";
		static const FString EZAttrib_OutOfOrderPacketsLostCount = "OutOfOrderPacketsLostCount";
		static const FString EZAttrib_OutOfOrderPacketsRecoveredCount = "OutOfOrderPacketsRecoveredCount";
		static const FString EZAttrib_OutOfOrderPacketsDuplicateCount = "OutOfOrderPacketsDuplicateCount";
		static const FString EZAttrib_CloseReasons = "CloseReasons";
		static const FString EZAttrib_FullCloseReasons = "FullCloseReasons";
		static const FString EZAttrib_ClientCloseReasons = "ClientCloseReasons";
		static const FString EZAttrib_FullClientCloseReasons = "FullClientCloseReasons";
		static const FString EZAttrib_RecoveredFaults = "RecoveredFaults";
		static const FString EZAttrib_FailedPingICMP = "FailedPingICMP";
		static const FString EZAttrib_FailedPingUDP = "FailedPingUDP";
		static const FString EZAttrib_Reason = "Reason";
		static const FString EZAttrib_Count = "Count";
		static const FString EZAttrib_IP = "IP";


		// Json writer subclass to allow us to avoid using a SharedPtr to write basic Json
		typedef TCondensedJsonPrintPolicy<TCHAR> FPrintPolicy;
		class FAnalyticsJsonWriter : public TJsonStringWriter<FPrintPolicy>
		{
		public:
			explicit FAnalyticsJsonWriter(FString* Out) : TJsonStringWriter<FPrintPolicy>(Out, 0)
			{
			}
		};


		FString CloseReasonsJsonStr;

		if (HeadCloseReasons.Num() > 0)
		{
			FAnalyticsJsonWriter CloseReasonsJsonWriter(&CloseReasonsJsonStr);

			CloseReasonsJsonWriter.WriteArrayStart();

			for (const FReasonCounter& CurCounter : HeadCloseReasons)
			{
				CloseReasonsJsonWriter.WriteObjectStart();

				CloseReasonsJsonWriter.WriteValue(EZAttrib_Reason, ToCStr(CurCounter.ReasonStr));
				CloseReasonsJsonWriter.WriteValue(EZAttrib_Count, CurCounter.Counter);

				CloseReasonsJsonWriter.WriteObjectEnd();
			}

			CloseReasonsJsonWriter.WriteArrayEnd();
			CloseReasonsJsonWriter.Close();
		}

		FString FullCloseReasonsJsonStr;

		if (FullCloseReasons.Num() > 0)
		{
			FAnalyticsJsonWriter FullCloseReasonsJsonWriter(&FullCloseReasonsJsonStr);

			FullCloseReasonsJsonWriter.WriteArrayStart();

			for (const FReasonCounter& CurCounter : FullCloseReasons)
			{
				FullCloseReasonsJsonWriter.WriteObjectStart();

				FullCloseReasonsJsonWriter.WriteValue(EZAttrib_Reason, ToCStr(CurCounter.ReasonStr));
				FullCloseReasonsJsonWriter.WriteValue(EZAttrib_Count, CurCounter.Counter);

				FullCloseReasonsJsonWriter.WriteObjectEnd();
			}

			FullCloseReasonsJsonWriter.WriteArrayEnd();
			FullCloseReasonsJsonWriter.Close();
		}


		FString ClientCloseReasonsJsonStr;

		if (HeadClientCloseReasons.Num() > 0)
		{
			FAnalyticsJsonWriter ClientCloseReasonsJsonWriter(&ClientCloseReasonsJsonStr);

			ClientCloseReasonsJsonWriter.WriteArrayStart();

			for (const FReasonCounter& CurCounter : HeadClientCloseReasons)
			{
				ClientCloseReasonsJsonWriter.WriteObjectStart();

				ClientCloseReasonsJsonWriter.WriteValue(EZAttrib_Reason, ToCStr(CurCounter.ReasonStr));
				ClientCloseReasonsJsonWriter.WriteValue(EZAttrib_Count, CurCounter.Counter);

				ClientCloseReasonsJsonWriter.WriteObjectEnd();
			}

			ClientCloseReasonsJsonWriter.WriteArrayEnd();
			ClientCloseReasonsJsonWriter.Close();
		}

		FString FullClientCloseReasonsJsonStr;

		if (FullClientCloseReasons.Num() > 0)
		{
			FAnalyticsJsonWriter FullClientCloseReasonsJsonWriter(&FullClientCloseReasonsJsonStr);

			FullClientCloseReasonsJsonWriter.WriteArrayStart();

			for (const FReasonCounter& CurCounter : FullClientCloseReasons)
			{
				FullClientCloseReasonsJsonWriter.WriteObjectStart();

				FullClientCloseReasonsJsonWriter.WriteValue(EZAttrib_Reason, ToCStr(CurCounter.ReasonStr));
				FullClientCloseReasonsJsonWriter.WriteValue(EZAttrib_Count, CurCounter.Counter);

				FullClientCloseReasonsJsonWriter.WriteObjectEnd();
			}

			FullClientCloseReasonsJsonWriter.WriteArrayEnd();
			FullClientCloseReasonsJsonWriter.Close();
		}


		FString RecoveredFaultsJsonStr;

		if (RecoveredFaults.Num() > 0)
		{
			FAnalyticsJsonWriter RecoveredFaultsJsonWriter(&RecoveredFaultsJsonStr);

			RecoveredFaultsJsonWriter.WriteArrayStart();

			for (TMap<FString, int32>::TConstIterator It(RecoveredFaults); It; ++It)
			{
				RecoveredFaultsJsonWriter.WriteObjectStart();

				RecoveredFaultsJsonWriter.WriteValue(EZAttrib_Reason, ToCStr(It.Key()));
				RecoveredFaultsJsonWriter.WriteValue(EZAttrib_Count, It.Value());

				RecoveredFaultsJsonWriter.WriteObjectEnd();
			}

			RecoveredFaultsJsonWriter.WriteArrayEnd();
			RecoveredFaultsJsonWriter.Close();
		}


		FString FailedPingAddressesICMPJsonStr;

		if (ProcessedFailedPingAddressesICMP.Num() > 0)
		{
			FAnalyticsJsonWriter FailedPingAddressesICMPJsonWriter(&FailedPingAddressesICMPJsonStr);

			FailedPingAddressesICMPJsonWriter.WriteArrayStart();

			for (const FFailedPingCounter& CurCounter : ProcessedFailedPingAddressesICMP)
			{
				FailedPingAddressesICMPJsonWriter.WriteObjectStart();

				FailedPingAddressesICMPJsonWriter.WriteValue(EZAttrib_IP, ToCStr(CurCounter.PingAddress));
				FailedPingAddressesICMPJsonWriter.WriteValue(EZAttrib_Count, CurCounter.Counter);

				FailedPingAddressesICMPJsonWriter.WriteObjectEnd();
			}

			FailedPingAddressesICMPJsonWriter.WriteArrayEnd();
			FailedPingAddressesICMPJsonWriter.Close();
		}

		FString FailedPingAddressesUDPJsonStr;

		if (ProcessedFailedPingAddressesUDP.Num() > 0)
		{
			FAnalyticsJsonWriter FailedPingAddressesUDPJsonWriter(&FailedPingAddressesUDPJsonStr);

			FailedPingAddressesUDPJsonWriter.WriteArrayStart();

			for (const FFailedPingCounter& CurCounter : ProcessedFailedPingAddressesUDP)
			{
				FailedPingAddressesUDPJsonWriter.WriteObjectStart();

				FailedPingAddressesUDPJsonWriter.WriteValue(EZAttrib_IP, ToCStr(CurCounter.PingAddress));
				FailedPingAddressesUDPJsonWriter.WriteValue(EZAttrib_Count, CurCounter.Counter);

				FailedPingAddressesUDPJsonWriter.WriteObjectEnd();
			}

			FailedPingAddressesUDPJsonWriter.WriteArrayEnd();
			FailedPingAddressesUDPJsonWriter.Close();
		}


		// IMPORTANT: Make sure the number of attributes is accurate when adding new analytics - for efficient allocation
		const int32 NumAttribs = 12;
		TArray<FAnalyticsEventAttribute> Attrs;

		Attrs.Reserve(NumAttribs);

		AppendAnalyticsEventAttributeArray(Attrs,
			EZAttrib_OutAckOnlyCount, OutAckOnlyCount,
			EZAttrib_OutKeepAliveCount, OutKeepAliveCount,
			EZAttrib_OutOfOrderPacketsLostCount, OutOfOrderPacketsLostCount,
			EZAttrib_OutOfOrderPacketsRecoveredCount, OutOfOrderPacketsRecoveredCount,
			EZAttrib_OutOfOrderPacketsDuplicateCount, OutOfOrderPacketsDuplicateCount);

		if (CloseReasonsJsonStr.Len() > 0)
		{
			AppendAnalyticsEventAttributeArray(Attrs, EZAttrib_CloseReasons, FJsonFragment(MoveTemp(CloseReasonsJsonStr)));
		}

		if (FullCloseReasonsJsonStr.Len() > 0)
		{
			AppendAnalyticsEventAttributeArray(Attrs, EZAttrib_FullCloseReasons, FJsonFragment(MoveTemp(FullCloseReasonsJsonStr)));
		}

		if (ClientCloseReasonsJsonStr.Len() > 0)
		{
			AppendAnalyticsEventAttributeArray(Attrs, EZAttrib_ClientCloseReasons, FJsonFragment(MoveTemp(ClientCloseReasonsJsonStr)));
		}

		if (FullClientCloseReasonsJsonStr.Len() > 0)
		{
			AppendAnalyticsEventAttributeArray(Attrs, EZAttrib_FullClientCloseReasons, FJsonFragment(MoveTemp(FullClientCloseReasonsJsonStr)));
		}

		if (RecoveredFaultsJsonStr.Len() > 0)
		{
			AppendAnalyticsEventAttributeArray(Attrs, EZAttrib_RecoveredFaults, FJsonFragment(MoveTemp(RecoveredFaultsJsonStr)));
		}

		if (FailedPingAddressesICMPJsonStr.Len() > 0)
		{
			AppendAnalyticsEventAttributeArray(Attrs, EZAttrib_FailedPingICMP, FJsonFragment(MoveTemp(FailedPingAddressesICMPJsonStr)));
		}

		if (FailedPingAddressesUDPJsonStr.Len() > 0)
		{
			AppendAnalyticsEventAttributeArray(Attrs, EZAttrib_FailedPingUDP, FJsonFragment(MoveTemp(FailedPingAddressesUDPJsonStr)));
		}

		Aggregator->AppendGameInstanceAttributes(Attrs);

		AnalyticsProvider->RecordEvent(EZEventName, MoveTemp(Attrs));
	}
}
