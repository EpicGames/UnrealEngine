// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimBlueprint.h"
#include "Animation/AnimInstance.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Animation/AnimBlueprintGeneratedClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprint)

#if WITH_EDITOR
#include "IAnimationBlueprintEditorModule.h"
#include "Settings/AnimBlueprintSettings.h"
#endif
#if WITH_EDITORONLY_DATA
#include "AnimationEditorUtils.h"
#endif

//////////////////////////////////////////////////////////////////////////
// UAnimBlueprint

UAnimBlueprint::UAnimBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = true;
#if WITH_EDITORONLY_DATA
	bRefreshExtensions = true;
#endif
	
#if WITH_EDITOR
	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Ensure that we are able to compile this anim BP by loading the compiler's module
		FModuleManager::Get().LoadModuleChecked("AnimGraph");
	}

	DefaultBindingClass = FindObject<UClass>(nullptr, TEXT("/Script/AnimGraph.AnimGraphNodeBinding_Base"));
#endif
}

UAnimBlueprintGeneratedClass* UAnimBlueprint::GetAnimBlueprintGeneratedClass() const
{
	UAnimBlueprintGeneratedClass* Result = Cast<UAnimBlueprintGeneratedClass>(*GeneratedClass);
	return Result;
}

UAnimBlueprintGeneratedClass* UAnimBlueprint::GetAnimBlueprintSkeletonClass() const
{
	UAnimBlueprintGeneratedClass* Result = Cast<UAnimBlueprintGeneratedClass>(*SkeletonGeneratedClass);
	return Result;
}

void UAnimBlueprint::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::Animation);

	Super::Serialize(Ar);

#if WITH_EDITOR
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
#endif
}

#if WITH_EDITOR

UClass* UAnimBlueprint::GetBlueprintClass() const
{
	return UAnimBlueprintGeneratedClass::StaticClass();
}

int32 UAnimBlueprint::FindOrAddGroup(FName GroupName)
{
	if (GroupName == NAME_None)
	{
		return INDEX_NONE;
	}
	else
	{
		// Look for an existing group
		for (int32 Index = 0; Index < Groups.Num(); ++Index)
		{
			if (Groups[Index].Name == GroupName)
			{
				return Index;
			}
		}

		// Create a new group
		MarkPackageDirty();
		FAnimGroupInfo& NewGroup = *(new (Groups) FAnimGroupInfo());
		NewGroup.Name = GroupName;

		return Groups.Num() - 1;
	}
}

UAnimBlueprint* UAnimBlueprint::FindRootAnimBlueprint(const UAnimBlueprint* DerivedBlueprint)
{
	UAnimBlueprint* ParentBP = nullptr;

	// Determine if there is an anim blueprint in the ancestry of this class
	for (UClass* ParentClass = DerivedBlueprint->ParentClass; ParentClass && (UObject::StaticClass() != ParentClass); ParentClass = ParentClass->GetSuperClass())
	{
		if (UAnimBlueprint* TestBP = Cast<UAnimBlueprint>(ParentClass->ClassGeneratedBy))
		{
			ParentBP = TestBP;
		}
	}

	return ParentBP;
}

UAnimBlueprint* UAnimBlueprint::GetParentAnimBlueprint(const UAnimBlueprint* DerivedBlueprint)
{
	UAnimBlueprint* ParentBP = nullptr;
	UClass* ParentClass = DerivedBlueprint->ParentClass;

	if (UAnimBlueprint* TestBP = Cast<UAnimBlueprint>(ParentClass->ClassGeneratedBy))
	{
		ParentBP = TestBP;
	}

	return ParentBP;
}

FAnimParentNodeAssetOverride* UAnimBlueprint::GetAssetOverrideForNode(FGuid NodeGuid, bool bIgnoreSelf) const
{
	TArray<UBlueprint*> Hierarchy;
	GetBlueprintHierarchyFromClass(GetAnimBlueprintGeneratedClass(), Hierarchy);

	for (int32 Idx = bIgnoreSelf ? 1 : 0 ; Idx < Hierarchy.Num() ; ++Idx)
	{
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Hierarchy[Idx]))
		{
			FAnimParentNodeAssetOverride* Override = AnimBlueprint->ParentAssetOverrides.FindByPredicate([NodeGuid](const FAnimParentNodeAssetOverride& Other)
			{
				return Other.ParentNodeGuid == NodeGuid;
			});

			if (Override)
			{
				return Override;
			}
		}
	}

	return nullptr;
}

bool UAnimBlueprint::GetAssetOverrides(TArray<FAnimParentNodeAssetOverride*>& OutOverrides)
{
	TArray<UBlueprint*> Hierarchy;
	GetBlueprintHierarchyFromClass(GetAnimBlueprintGeneratedClass(), Hierarchy);

	for (UBlueprint* Blueprint : Hierarchy)
	{
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
		{
			for (FAnimParentNodeAssetOverride& Override : AnimBlueprint->ParentAssetOverrides)
			{
				bool OverrideExists = OutOverrides.ContainsByPredicate([Override](const FAnimParentNodeAssetOverride* Other)
				{
					return Override.ParentNodeGuid == Other->ParentNodeGuid;
				});

				if (!OverrideExists)
				{
					OutOverrides.Add(&Override);
				}
			}
		}
	}

	return OutOverrides.Num() > 0;
}

void UAnimBlueprint::PostLoad()
{
	LLM_SCOPE(ELLMTag::Animation);

	Super::PostLoad();
#if WITH_EDITOR
	// Validate animation overrides
	UAnimBlueprintGeneratedClass* AnimBPGeneratedClass = GetAnimBlueprintGeneratedClass();
	
	if (AnimBPGeneratedClass)
	{
		// If there is no index for the guid, remove the entry.
		ParentAssetOverrides.RemoveAll([AnimBPGeneratedClass](const FAnimParentNodeAssetOverride& Element)
		{
			if (!AnimBPGeneratedClass->GetNodePropertyIndexFromGuid(Element.ParentNodeGuid, EPropertySearchMode::Hierarchy))
			{
				return true;
			}

			return false;
		});
	}
#endif
}

bool UAnimBlueprint::CanAlwaysRecompileWhilePlayingInEditor() const
{
	return true;
}

bool UAnimBlueprint::FindDiffs(const UBlueprint* OtherBlueprint, FDiffResults& Results) const
{
	const UAnimBlueprint* OtherAnimBP = Cast<UAnimBlueprint>(OtherBlueprint);
	if (!OtherAnimBP)
	{
		return false;
	}

	// Anim BPs should diff correctly, as all the info is stored in graphs or the parent
	return true;
}

void UAnimBlueprint::SetObjectBeingDebugged(UObject* NewObject)
{
	// Look for any linked instances and set them up too if they are not already open in an editor
	AnimationEditorUtils::SetupDebugLinkedAnimInstances(this, NewObject);

	Super::SetObjectBeingDebugged(NewObject);
}

bool UAnimBlueprint::SupportsEventGraphs() const
{
	return GetDefault<UAnimBlueprintSettings>()->bAllowEventGraphs;
}

bool UAnimBlueprint::SupportsAnimLayers() const
{
	return true;
}

bool UAnimBlueprint::SupportsDelegates() const
{
	return GetDefault<UAnimBlueprintSettings>()->bAllowDelegates;
}

bool UAnimBlueprint::SupportsMacros() const
{
	return GetDefault<UAnimBlueprintSettings>()->bAllowMacros;
}

bool UAnimBlueprint::SupportsInputEvents() const
{
	// Animation blueprints don't really support input events. You used to be able to place
	// the input event nodes in the anim graphs and they would just not work, causing an ensure.
	// To keep backwards compatibility we will allow users to enable them in case
	// their project somehow relies on it in some custom node extension, but
	// for new projects it should not be allowed. 
	if (GetDefault<UAnimBlueprintSettings>()->bSupportInputEventsForBackwardsCompatibility)
	{
		return Super::SupportsInputEvents();
	}
	
	return false;	
}

bool UAnimBlueprint::AllowFunctionOverride(const UFunction* const InFunction) const
{
	check(InFunction);

	if (!GetDefault<UAnimBlueprintSettings>()->bRestrictBaseFunctionOverrides)
	{
		return true;
	}

	UClass* OwnerClass = InFunction->GetOwnerClass();
	if (OwnerClass == UAnimInstance::StaticClass())
	{
		return GetDefault<UAnimBlueprintSettings>()->BaseFunctionOverrideAllowList.Contains(InFunction->GetFName());
	}
	else
	{
		return true;
	}
}

void UAnimBlueprint::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	// Make sure we add / remove linked anim layer sharing blueprint extension
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimBlueprint, bEnableLinkedAnimLayerInstanceSharing))
	{
		bRefreshExtensions = true;
	}
}

void UAnimBlueprint::GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	Super::GetTypeActions(ActionRegistrar);

	const IAnimationBlueprintEditorModule& AnimationBlueprintEditorModule = FModuleManager::LoadModuleChecked<IAnimationBlueprintEditorModule>(TEXT("AnimationBlueprintEditor"));
	AnimationBlueprintEditorModule.GetTypeActions(ActionRegistrar);
}

void UAnimBlueprint::GetInstanceActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	Super::GetInstanceActions(ActionRegistrar);

	const IAnimationBlueprintEditorModule& AnimationBlueprintEditorModule = FModuleManager::LoadModuleChecked<IAnimationBlueprintEditorModule>(TEXT("AnimationBlueprintEditor"));
	AnimationBlueprintEditorModule.GetInstanceActions(this, ActionRegistrar);
}

#endif

USkeletalMesh* UAnimBlueprint::GetPreviewMesh(bool bFindIfNotSet/*=false*/)
{
#if WITH_EDITORONLY_DATA
	if (!PreviewSkeletalMesh.IsValid())
	{
		PreviewSkeletalMesh.LoadSynchronous();
	}
	return PreviewSkeletalMesh.Get();
#else
	return nullptr;
#endif
}

USkeletalMesh* UAnimBlueprint::GetPreviewMesh() const
{
#if WITH_EDITORONLY_DATA
	if (!PreviewSkeletalMesh.IsValid())
	{
		PreviewSkeletalMesh.LoadSynchronous();
	}
	return PreviewSkeletalMesh.Get();
#else
	return nullptr;
#endif
}

void UAnimBlueprint::SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty/*=true*/)
{
#if WITH_EDITORONLY_DATA
	if(bMarkAsDirty)
	{
		Modify();
	}
	PreviewSkeletalMesh = PreviewMesh;
#endif
}

void UAnimBlueprint::SetPreviewAnimationBlueprint(UAnimBlueprint* InPreviewAnimationBlueprint)
{
#if WITH_EDITORONLY_DATA
	Modify();
	PreviewAnimationBlueprint = InPreviewAnimationBlueprint;
#endif
}

UAnimBlueprint* UAnimBlueprint::GetPreviewAnimationBlueprint() const
{
#if WITH_EDITORONLY_DATA
	if (!PreviewAnimationBlueprint.IsValid())
	{
		PreviewAnimationBlueprint.LoadSynchronous();
	}
	return PreviewAnimationBlueprint.Get();
#else
	return nullptr;
#endif
}

void UAnimBlueprint::SetPreviewAnimationBlueprintApplicationMethod(EPreviewAnimationBlueprintApplicationMethod InMethod) 
{ 
#if WITH_EDITORONLY_DATA
	Modify();
	PreviewAnimationBlueprintApplicationMethod = InMethod; 
#endif
}

EPreviewAnimationBlueprintApplicationMethod UAnimBlueprint::GetPreviewAnimationBlueprintApplicationMethod() const 
{ 
#if WITH_EDITORONLY_DATA
	return PreviewAnimationBlueprintApplicationMethod; 
#else
	return EPreviewAnimationBlueprintApplicationMethod::LinkedLayers;
#endif
}

void UAnimBlueprint::SetPreviewAnimationBlueprintTag(FName InTag) 
{ 
#if WITH_EDITORONLY_DATA
	Modify();
	PreviewAnimationBlueprintTag = InTag; 
#endif
}

FName UAnimBlueprint::GetPreviewAnimationBlueprintTag() const 
{ 
#if WITH_EDITORONLY_DATA
	return PreviewAnimationBlueprintTag; 
#else
	return NAME_None;
#endif
}

bool UAnimBlueprint::IsObjectBeingDebugged(const UObject* Object) const
{
#if WITH_EDITOR
	// Only root anim BPs can have anim graphs and be debugged
	const UAnimBlueprint* RootBP = UAnimBlueprint::FindRootAnimBlueprint(this);
	const UAnimBlueprint* DebugBP = RootBP ? RootBP : this;
	
	return DebugBP->GetObjectBeingDebugged() == Object;
#else
	return false;
#endif
}

FAnimBlueprintDebugData* UAnimBlueprint::GetDebugData() const
{
#if WITH_EDITORONLY_DATA
	// Only root anim BPs can have anim graphs and be debugged
	const UAnimBlueprint* RootBP = UAnimBlueprint::FindRootAnimBlueprint(this);
	const UAnimBlueprint* DebugBP = RootBP ? RootBP : this;
	UAnimBlueprintGeneratedClass* AnimClass = DebugBP->GetAnimBlueprintGeneratedClass();

	return AnimClass ? &AnimClass->GetAnimBlueprintDebugData() : nullptr;
#else
	return nullptr;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
bool UAnimBlueprint::IsCompatible(const UAnimBlueprint* InAnimBlueprint) const
{
	if(InAnimBlueprint->bIsTemplate)
	{
		// Directionality here, templates are compatible with all anim BPs, but not necessarily vice versa
		return true;
	}
	
	if(InAnimBlueprint->BlueprintType == BPTYPE_Interface)
	{
		// Interfaces dont bother with skeleton checks - assume compatibility is driven by the interface machinery
		return true;
	}

	return (TargetSkeleton != nullptr && TargetSkeleton->IsCompatibleForEditor(InAnimBlueprint->TargetSkeleton));
}

bool UAnimBlueprint::IsCompatibleByAssetString(const FString& InSkeletonAsset, bool bInIsTemplate, bool bInIsInterface) const
{
	if(bInIsTemplate)
	{
		// Directionality here, templates are compatible with all anim BPs, but not necessarily vice versa
		return true;
	}
	
	if(bInIsInterface)
	{
		// Interfaces dont bother with skeleton checks - assume compatibility is driven by the interface machinery
		return true;
	}

	return (TargetSkeleton != nullptr && TargetSkeleton->IsCompatibleForEditor(InSkeletonAsset));
}

void UAnimBlueprint::NotifyGraphRenamed(class UEdGraph* Graph, FName OldName, FName NewName)
{
	OnGraphRenamedEvent().Broadcast(Graph, OldName, NewName);
}
#endif
