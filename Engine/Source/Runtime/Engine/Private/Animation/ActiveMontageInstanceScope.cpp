// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/ActiveMontageInstanceScope.h"
#include "Animation/AnimNodeMessages.h"

namespace UE::Anim {

IMPLEMENT_NOTIFY_CONTEXT_INTERFACE(FAnimNotifyMontageInstanceContext)

FAnimNotifyMontageInstanceContext::FAnimNotifyMontageInstanceContext(const int32 InMontageInstanceID)
	: MontageInstanceID(InMontageInstanceID)
{
}
	
}	// namespace UE::Anim
