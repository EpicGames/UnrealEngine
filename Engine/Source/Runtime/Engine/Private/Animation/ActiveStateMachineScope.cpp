// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/ActiveStateMachineScope.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNode_StateMachine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActiveStateMachineScope)

FEncounteredStateMachineStack::FEncounteredStateMachineStack(const FEncounteredStateMachineStack& ParentStack, int32 InStateMachineIndex, int32 InStateIndex) :
	StateStack(ParentStack.StateStack)
{
	StateStack.Emplace(InStateMachineIndex, InStateIndex);
}

FEncounteredStateMachineStack::FEncounteredStateMachineStack(int32 InStateMachineIndex, int32 InStateIndex)
{
	StateStack.Emplace(InStateMachineIndex, InStateIndex);
}

FEncounteredStateMachineStack FEncounteredStateMachineStack::InitStack(const FEncounteredStateMachineStack& ParentStack, int32 InStateMachineIndex, int32 InStateIndex)
{
	return FEncounteredStateMachineStack(ParentStack, InStateMachineIndex, InStateIndex); 
}

FEncounteredStateMachineStack FEncounteredStateMachineStack::InitStack(int32 InStateMachineIndex, int32 InStateIndex)
{
	return FEncounteredStateMachineStack(InStateMachineIndex, InStateIndex);
}



namespace UE { namespace Anim {

IMPLEMENT_NOTIFY_CONTEXT_INTERFACE(FAnimNotifyStateMachineContext)

FAnimNotifyStateMachineContext::FAnimNotifyStateMachineContext(const FEncounteredStateMachineStack& InEncounteredStateMachines)
	: EncounteredStateMachines(InEncounteredStateMachines)
{
}

bool FAnimNotifyStateMachineContext::IsStateMachineInContext(int32 StateMachineIndex) const
{
	for (const FEncounteredStateMachineStack::FStateMachineEntry& Entry : EncounteredStateMachines.StateStack)
	{
		if (Entry.StateMachineIndex == StateMachineIndex)
		{
			return true; 
		}
	}
	return false; 
}

bool FAnimNotifyStateMachineContext::IsStateInStateMachineInContext(int32 StateMachineIndex, int32 StateIndex) const
{
	for (const FEncounteredStateMachineStack::FStateMachineEntry& Entry : EncounteredStateMachines.StateStack)
	{
		if (Entry.StateMachineIndex == StateMachineIndex && Entry.StateIndex == StateIndex)
		{
			return true;
		}
	}
	return false;
}

IMPLEMENT_ANIMGRAPH_MESSAGE(FActiveStateMachineScope);

FActiveStateMachineScope::FActiveStateMachineScope(const FAnimationBaseContext& InContext, FAnimNode_StateMachine* StateMachine, int32 InStateIndex)
	: ActiveStateMachines([&InContext, StateMachine, InStateIndex]()
	{
		const int32 StateMachineIndex = GetStateMachineIndex(StateMachine, InContext);
		if (FActiveStateMachineScope* ParentStateMachineScope = InContext.GetMessage<FActiveStateMachineScope>())
		{
			return FEncounteredStateMachineStack::InitStack(ParentStateMachineScope->ActiveStateMachines, StateMachineIndex, InStateIndex);
		}
		else
		{
			return FEncounteredStateMachineStack::InitStack(StateMachineIndex, InStateIndex);
		}
	}())
{
}

int32 FActiveStateMachineScope::GetStateMachineIndex(FAnimNode_StateMachine* StateMachine, const FAnimationBaseContext& Context)
{
	if (Context.AnimInstanceProxy && Context.AnimInstanceProxy->GetAnimClassInterface())
	{
		const int32 NumProperties = Context.AnimInstanceProxy->GetAnimClassInterface()->GetAnimNodeProperties().Num();
		const int32 StateMachineIndex = NumProperties - 1 - StateMachine->GetNodeIndex();
		return StateMachineIndex;
	}
	return INDEX_NONE;
}

TUniquePtr<const IAnimNotifyEventContextDataInterface> FActiveStateMachineScope::MakeUniqueEventContextData() const
{
	return MakeUnique<const FAnimNotifyStateMachineContext>(ActiveStateMachines);
}
	
}}	// namespace UE::Anim
