// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AimOffsetBlendSpace.cpp: AimOffsetBlendSpace functionality
=============================================================================*/ 

#include "Animation/AimOffsetBlendSpace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AimOffsetBlendSpace)

UAimOffsetBlendSpace::UAimOffsetBlendSpace(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UAimOffsetBlendSpace::IsValidAdditiveType(EAdditiveAnimationType AdditiveType) const
{
	return (AdditiveType == AAT_RotationOffsetMeshSpace);
}

bool UAimOffsetBlendSpace::IsValidAdditive() const
{
	return ContainsMatchingSamples(AAT_RotationOffsetMeshSpace);
}

