// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AimOffsetBlendSpace1D.cpp: AimOffsetBlendSpace functionality
=============================================================================*/ 

#include "Animation/AimOffsetBlendSpace1D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AimOffsetBlendSpace1D)

UAimOffsetBlendSpace1D::UAimOffsetBlendSpace1D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UAimOffsetBlendSpace1D::IsValidAdditiveType(EAdditiveAnimationType AdditiveType) const
{
	return (AdditiveType == AAT_RotationOffsetMeshSpace);
}

bool UAimOffsetBlendSpace1D::IsValidAdditive() const
{
	return ContainsMatchingSamples(AAT_RotationOffsetMeshSpace);
}

