// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimAttributes.h"

const FName UE::Anim::FAttributes::Pose("Pose");
const FName UE::Anim::FAttributes::Curves("Curves");
const FName UE::Anim::FAttributes::Attributes("Attributes");
