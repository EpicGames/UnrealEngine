// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimBank.h"
#include "Animation/AnimBoneCompressionCodec.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "AnimationRuntime.h"
#include "AnimationUtils.h"
#include "AnimEncoding.h"
#include "Async/ParallelFor.h"
#include "DerivedDataCacheInterface.h"
#include "EngineUtils.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Serialization/ArchiveCrc32.h"
#include "Logging/MessageLog.h"
#include "Misc/App.h"
#include "ReferenceSkeleton.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "SkeletalRenderPublic.h"
#include "SkinningSceneExtensionProxy.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/AnimObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBank)

#if WITH_EDITOR
#include "Animation/AnimationSequenceCompiler.h"
#include "Animation/AnimBankCompiler.h"
#include "Async/AsyncWork.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "Experimental/Misc/ExecutionResource.h"
#include "Serialization/MemoryHasher.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#endif

#include "Components/InstancedSkinnedMeshComponent.h"

#define LOCTEXT_NAMESPACE "AnimBank"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

DEFINE_LOG_CATEGORY(LogAnimBank);

static TAutoConsoleVariable<int32> CVarMemoryForAnimBankAssetCompile(
	TEXT("Memory.MemoryForAnimBankAssetCompile"),
	512,
	TEXT("Memory in MiB set aside for a AnimBank asset compile job\n"),
	ECVF_Default);

#if WITH_EDITOR

UAnimBank::FOnDependenciesChanged UAnimBank::OnDependenciesChanged;

static inline void DecomposeTransform(const FTransform& Transform, FVector3f& OutTranslation, FQuat4f& OutRotation)
{
	// Get Translation
	OutTranslation = (FVector3f)Transform.GetTranslation();

	// Get Rotation 
	OutRotation = (FQuat4f)Transform.GetRotation();
}

static inline void DecomposeTransforms(
	const TArray<FTransform>& Transforms,
	TArray<FVector3f>& OutTranslations,
	TArray<FQuat4f>& OutRotations)
{
	const int32 NumTransforms = Transforms.Num();
	for (int32 Index = 0; Index < NumTransforms; ++Index)
	{
		FVector3f Translation;
		FQuat4f Rotation;
		DecomposeTransform(Transforms[Index], Translation, Rotation);
		OutTranslations.Emplace(MoveTemp(Translation));
		OutRotations.Emplace(MoveTemp(Rotation));
	}
}

static inline uint32 BankFlagsFromSequence(const FAnimBankSequence& BankSequence)
{
	uint32 Flags = ANIM_BANK_FLAG_NONE;

	if (BankSequence.bLooping)
	{
		Flags |= ANIM_BANK_FLAG_LOOPING;
	}

	if (BankSequence.bAutoStart)
	{
		Flags |= ANIM_BANK_FLAG_AUTOSTART;
	}

	return Flags;
}

class FAnimBankAsyncBuildWorker : public FNonAbandonableTask
{
	FAnimBankBuildAsyncCacheTask* Owner;
	FIoHash IoHash;

public:
	FAnimBankAsyncBuildWorker(FAnimBankBuildAsyncCacheTask* InOwner, const FIoHash& InIoHash)
	: Owner(InOwner)
	, IoHash(InIoHash)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAnimBankAsyncBuildWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	void DoWork();
};

struct FAnimBankAsyncBuildTask : public FAsyncTask<FAnimBankAsyncBuildWorker>
{
	FAnimBankAsyncBuildTask(FAnimBankBuildAsyncCacheTask* InOwner, const FIoHash& InIoHash)
	: FAsyncTask<FAnimBankAsyncBuildWorker>(InOwner, InIoHash)
	{
	}
};

class FAnimBankBuildAsyncCacheTask
{
public:
	FAnimBankBuildAsyncCacheTask(
		const FIoHash& InKeyHash,
		FAnimBankData* InData,
		UAnimBank& InDisplacedMesh,
		const ITargetPlatform* TargetPlatform
	);

	inline void Wait()
	{
		if (bIsWaitingOnCompilation)
		{
			WaitForDependenciesAndBeginCache();
		}

		if (BuildTask != nullptr)
		{
			BuildTask->EnsureCompletion();
		}

		Owner.Wait();
	}

	inline bool WaitWithTimeout(float TimeLimitSeconds)
	{
		if (bIsWaitingOnCompilation)
		{
			if (!WaitForDependenciesAndBeginCacheWithTimeout(TimeLimitSeconds))
			{
				return false;
			}
		}

		if (BuildTask != nullptr && !BuildTask->WaitCompletionWithTimeout(TimeLimitSeconds))
		{
			return false;
		}

		return Owner.Poll();
	}

	inline bool Poll()
	{
		if (bIsWaitingOnCompilation)
		{
			BeginCacheIfDependenciesAreFree();
			return false;
		}

		if (BuildTask && !BuildTask->IsDone())
		{
			return false;
		}

		return Owner.Poll();
	}

	inline void Cancel()
	{
		// Cancel the waiting on the build
		bIsWaitingOnCompilation = false;

		if (BuildTask)
		{
			BuildTask->Cancel();
		}

		Owner.Cancel();
	}

	void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority);

private:
	bool ShouldWaitForCompilation() const;

	void BeginCacheIfDependenciesAreFree();
	void WaitForDependenciesAndBeginCache();
	bool WaitForDependenciesAndBeginCacheWithTimeout(float TimeLimitSeconds);

	void BeginCache(const FIoHash& KeyHash);
	void EndCache(UE::DerivedData::FCacheGetValueResponse&& Response);
	bool BuildData(const UE::FSharedString& Name, const UE::DerivedData::FCacheKey& Key);

private:
	friend class FAnimBankAsyncBuildWorker;
	TUniquePtr<FAnimBankAsyncBuildTask> BuildTask;
	FAnimBankData* Data;
	TWeakObjectPtr<UAnimBank> WeakAnimBank;

	TArray<FAnimBankSequence> BankItems;

	const ITargetPlatform* TargetPlatform = nullptr;

	UE::DerivedData::FRequestOwner Owner;
	TRefCountPtr<IExecutionResource> ExecutionResource;

	bool bIsWaitingOnCompilation;
	FIoHash KeyHash;
};

void FAnimBankAsyncBuildWorker::DoWork()
{
	using namespace UE::DerivedData;
	if (UAnimBank* Bank = Owner->WeakAnimBank.Get())
	{
		// Grab any execution resources currently assigned to this worker so that we maintain
		// concurrency limit and memory pressure until the whole multi-step task is done.
		Owner->ExecutionResource = FExecutionResourceContext::Get();

		static const FCacheBucket Bucket("AnimBank");
		GetCache().GetValue({ {{Bank->GetPathName()}, {Bucket, IoHash}} }, Owner->Owner,
			[Task = Owner](FCacheGetValueResponse&& Response)
			{
				Task->EndCache(MoveTemp(Response));
			}
		);
	}
}

FAnimBankBuildAsyncCacheTask::FAnimBankBuildAsyncCacheTask(
	const FIoHash& InKeyHash,
	FAnimBankData* InData,
	UAnimBank& InBank,
	const ITargetPlatform* InTargetPlatform
)
: Data(InData)
, WeakAnimBank(&InBank)
, BankItems(InBank.Sequences)
, TargetPlatform(InTargetPlatform)
// Once we pass the BeginCache throttling gate, we want to finish as fast as possible
// to avoid holding on to memory for a long time. We use the high priority since it will go fast,
// but also it will avoid starving the critical threads in the subsequent task.
, Owner(UE::DerivedData::EPriority::High)
, bIsWaitingOnCompilation(ShouldWaitForCompilation())
, KeyHash(InKeyHash)
{
	/**
	 * Unfortunately our async builds are not made to handle the assets that use data from other assets
	 * This will delay the start of the actual cache until the build of the sequences is done
	 * This will fix a race condition with the sequence build without blocking the game thread by default.
	 * Note: This is not a perfect solution since it also delays the DDC data pull.
	 */
	if (!bIsWaitingOnCompilation)
	{
		BeginCache(InKeyHash);
	}
}

bool FAnimBankBuildAsyncCacheTask::ShouldWaitForCompilation() const 
{
	if (UAnimBank* Bank = WeakAnimBank.Get())
	{
		for (FAnimBankSequence& BankSequence : Bank->Sequences)
		{
			if (!IsValid(BankSequence.Sequence))
			{
				continue;
			}

			// If the sequence is still waiting for a post load call, let it build its stuff first to avoid blocking the Game Thread
			if (BankSequence.Sequence->HasAnyFlags(RF_NeedPostLoad) || BankSequence.Sequence->IsCompiling() || !BankSequence.Sequence->CanBeCompressed())
			{
				return true;
			}
		}
	}

	return false;
}

void FAnimBankBuildAsyncCacheTask::BeginCacheIfDependenciesAreFree()
{
	if (UAnimBank* Bank = WeakAnimBank.Get())
	{
		if (!ShouldWaitForCompilation())
		{
			bIsWaitingOnCompilation = false;
			BeginCache(KeyHash);
		}
	}
	else
	{
		bIsWaitingOnCompilation = false;
	}
}

void FAnimBankBuildAsyncCacheTask::WaitForDependenciesAndBeginCache()
{
	if (UAnimBank* Bank = WeakAnimBank.Get())
	{
		for (FAnimBankSequence& BankSequence : Bank->Sequences)
		{
			if (!BankSequence.Sequence)
			{
				continue;
			}

			if (BankSequence.Sequence->HasAnyFlags(RF_NeedPostLoad))
			{
				BankSequence.Sequence->ConditionalPostLoad();
			}

			UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation({ BankSequence.Sequence.Get() });
		}

		bIsWaitingOnCompilation = false;
		BeginCache(KeyHash);
	}
	else
	{
		bIsWaitingOnCompilation = false;
	}
}

bool FAnimBankBuildAsyncCacheTask::WaitForDependenciesAndBeginCacheWithTimeout(float TimeLimitSeconds)
{
	if (UAnimBank* Bank = WeakAnimBank.Get())
	{
		for (FAnimBankSequence& BankSequence : Bank->Sequences)
		{
			if (!BankSequence.Sequence || !BankSequence.Sequence->IsCompiling())
			{
				continue;
			}

			if (!BankSequence.Sequence->WaitForAsyncTasks(TimeLimitSeconds))
			{
				return false;
			}
		}
	}

	// Performs any necessary cleanup now that the async task (if any) is complete
	WaitForDependenciesAndBeginCache();

	return true;
}

void FAnimBankBuildAsyncCacheTask::BeginCache(const FIoHash& InKeyHash)
{
	using namespace UE::DerivedData;

	if (UAnimBank* Bank = WeakAnimBank.Get())
	{
		for (FAnimBankSequence& BankSequence : Bank->Sequences)
		{
			UAnimSequence* Sequence = BankSequence.Sequence;
			if (!IsValid(Sequence))
			{
				continue;
			}

			Sequence->BeginCacheDerivedData(TargetPlatform);
		}

		// Queue this launch through the thread pool so that we benefit from fair scheduling and memory throttling
		FQueuedThreadPool* ThreadPool = FAnimBankCompilingManager::Get().GetThreadPool();
		EQueuedWorkPriority BasePriority = FAnimBankCompilingManager::Get().GetBasePriority(Bank);

		int64 RequiredMemory = 1024 * 1024 * CVarMemoryForAnimBankAssetCompile.GetValueOnAnyThread();

		check(BuildTask == nullptr);
		BuildTask = MakeUnique<FAnimBankAsyncBuildTask>(this, InKeyHash);
		BuildTask->StartBackgroundTask(ThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, RequiredMemory, TEXT("AnimBank"));
	}
}

void FAnimBankBuildAsyncCacheTask::EndCache(UE::DerivedData::FCacheGetValueResponse&& Response)
{
	using namespace UE::DerivedData;

	if (Response.Status == EStatus::Ok)
	{
		Owner.LaunchTask(TEXT("AnimBankSerialize"), [this, Value = MoveTemp(Response.Value)]
		{
			// Release execution resource as soon as the task is done
			ON_SCOPE_EXIT { ExecutionResource = nullptr; };

			if (UAnimBank* Bank = WeakAnimBank.Get())
			{
				FSharedBuffer RecordData = Value.GetData().Decompress();
				FMemoryReaderView Ar(RecordData, /*bIsPersistent*/ true);
				Ar << *Data;

				// The initialization of the resources is done by FAnimBankCompilingManager to avoid race conditions
			}
		});
	}
	else if (Response.Status == EStatus::Error)
	{
		Owner.LaunchTask(TEXT("AnimBankBuild"), [this, Name = Response.Name, Key = Response.Key]
		{
			// Release execution resource as soon as the task is done
			ON_SCOPE_EXIT { ExecutionResource = nullptr; };

			if (!BuildData(Name, Key))
			{
				return;
			}

			if (UAnimBank* Bank = WeakAnimBank.Get())
			{
				TArray64<uint8> RecordData;
				FMemoryWriter64 Ar(RecordData, /*bIsPersistent*/ true);
				Ar << *Data;

				GetCache().PutValue({ {Name, Key, FValue::Compress(MakeSharedBufferFromArray(MoveTemp(RecordData)))} }, Owner);

				// The initialization of the resources is done by FAnimBankCompilingManager to avoid race conditions
			}
		});
	}
	else
	{
		// Release execution resource as soon as the task is done
		ExecutionResource = nullptr;
	}
}

static void GetRefBoneGlobalSpace(const FReferenceSkeleton& RefSkeleton, TArray<FTransform>& Transforms)
{
	Transforms.Reset();

	const TArray<FTransform>& BoneSpaceTransforms = RefSkeleton.GetRawRefBonePose(); // Get only raw bones (no virtual)

	const int32 NumTransforms = BoneSpaceTransforms.Num();
	Transforms.SetNumUninitialized(NumTransforms);

	for (int32 BoneIndex = 0; BoneIndex < NumTransforms; ++BoneIndex)
	{
		// Initialize to identity since some of them don't have tracks
		int32 IterBoneIndex = BoneIndex;
		FTransform ComponentSpaceTransform = BoneSpaceTransforms[BoneIndex];

		do
		{
			const int32 ParentIndex = RefSkeleton.GetRawParentIndex(IterBoneIndex); // Get only raw bones (no virtual)
			if (ParentIndex != INDEX_NONE)
			{
				ComponentSpaceTransform *= BoneSpaceTransforms[ParentIndex];
			}

			IterBoneIndex = ParentIndex;
		}
		while (RefSkeleton.IsValidIndex(IterBoneIndex));

		Transforms[BoneIndex] = ComponentSpaceTransform;
	}
}

bool FAnimBankBuildAsyncCacheTask::BuildData(const UE::FSharedString& Name, const UE::DerivedData::FCacheKey& Key)
{
	using namespace UE::DerivedData;

	UAnimBank* AnimBank = WeakAnimBank.Get();
	if (!AnimBank)
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankBuildAsyncCacheTask::BuildData);

	*Data = {};

	if (!IsValid(AnimBank->Asset))
	{
		UE_LOGF(LogAnimBank, Error, "Cannot find a valid skinned asset to build the animation bank asset.");
		return false;
	}

	UE::AnimBank::BuildSkinnedAssetMapping(*AnimBank->Asset.Get(), Data->Mapping);

	const FBoxSphereBounds AssetBounds = AnimBank->Asset->GetBounds();

	const FReferenceSkeleton& MeshRefSkeleton = AnimBank->Asset->GetRefSkeleton();
	const TArray<FTransform>& MeshLocalRefPose = MeshRefSkeleton.GetRawRefBonePose();
	const int32 NumMeshBones = MeshRefSkeleton.GetRawBoneNum();

	Data->Entries.SetNum(AnimBank->Sequences.Num());
	for (int32 ItemIndex = 0; ItemIndex < AnimBank->Sequences.Num(); ++ItemIndex)
	{
		if (Owner.IsCanceled())
		{
			return false;
		}

		FAnimBankEntry& BankEntry = Data->Entries[ItemIndex];

		FAnimBankSequence& BankSequence = AnimBank->Sequences[ItemIndex];
		UAnimSequence* Sequence = BankSequence.Sequence.Get();
		if (Sequence == nullptr)
		{
			continue;
		}

		Sequence->FinishAsyncTasks();

		UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = Sequence->GetCompressedData(TargetPlatform);
		const FCompressedAnimSequence& PlatformCompressedData = CompressedAnimSequence.Get();
		// We should always have compressed data at this point
		if (!PlatformCompressedData.IsValid(Sequence) || PlatformCompressedData.BoneCompressionCodec == nullptr)
		{
			UE_LOGF(LogAnimBank, Error, "Animation bank referenced sequence is missing compressed data!");
			return false;
		}

		BankEntry.FrameCount = Sequence->GetSamplingFrameRate().AsFrameTime(Sequence->GetPlayLength()).RoundToFrame().Value;
		check(BankEntry.FrameCount > 0);

		BankEntry.KeyCount = BankEntry.FrameCount * NumMeshBones;//  Sequence->GetNumberOfSampledKeys();
		check(BankEntry.KeyCount > 0);

		BankEntry.Flags = BankFlagsFromSequence(BankSequence);
		BankEntry.Position = BankSequence.Position;
		BankEntry.PlayRate = BankSequence.PlayRate;

		TArray<FTransform> SampledGlobalMeshPose;
		SampledGlobalMeshPose.SetNumUninitialized(NumMeshBones);

		BankEntry.PositionKeys.SetNumUninitialized(BankEntry.KeyCount);
		BankEntry.RotationKeys.SetNumUninitialized(BankEntry.KeyCount);

		// Initialize bounds to the mesh vertex positions in reference pose.
		FVector3f AnimatedBoundsMin(AssetBounds.Origin - AssetBounds.BoxExtent);
		FVector3f AnimatedBoundsMax(AssetBounds.Origin + AssetBounds.BoxExtent);

		VectorRegister4Float VecAnimatedBoundsMin = VectorLoadFloat3(&AnimatedBoundsMin.X);
		VectorRegister4Float VecAnimatedBoundsMax = VectorLoadFloat3(&AnimatedBoundsMax.X);

		TArray<FBoneIndexType> BoneIndices;
		BoneIndices.SetNumUninitialized(NumMeshBones);
		for (int32 Index = 0; Index < NumMeshBones; ++Index)
		{
			BoneIndices[Index] = (FBoneIndexType)Index;
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankBuildAsyncCacheTask::BuildData);
			FBoneContainer BoneContainer(BoneIndices, UE::Anim::FCurveFilterSettings(), *AnimBank->Asset);

			int32 KeyIndex = 0;

			for (uint32 Frame = 0; Frame < BankEntry.FrameCount; ++Frame)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankBuildAsyncCacheTask::Frame);
				// Some paths in the decompression code use mem stack, so make sure we put a mark here.
				FMemMark Mark(FMemStack::Get());

				const double SeekTime = Sequence->GetSamplingFrameRate().AsSeconds(FFrameTime(int32(Frame)));

				FCompactPose AnimPose;
				AnimPose.SetBoneContainer(&BoneContainer);
				FBlendedCurve AnimCurve;
				AnimCurve.InitFrom(BoneContainer);
				UE::Anim::FStackAttributeContainer AnimAttributes;

				FAnimExtractContext ExtractionContext(SeekTime);
				FAnimationPoseData AnimationPoseData(AnimPose, AnimCurve, AnimAttributes);
				Sequence->GetAnimationPose(AnimationPoseData, ExtractionContext);

				for (int32 MeshBoneIndex = 0; MeshBoneIndex < NumMeshBones; ++MeshBoneIndex)
				{
					const FCompactPoseBoneIndex CompactIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
					const int32 ParentIndex = MeshRefSkeleton.GetParentIndex(MeshBoneIndex);
					if (SampledGlobalMeshPose.IsValidIndex(ParentIndex) && ParentIndex < MeshBoneIndex)
					{
						SampledGlobalMeshPose[MeshBoneIndex] = AnimationPoseData.GetPose()[CompactIndex] * SampledGlobalMeshPose[ParentIndex];
					}
					else
					{
						SampledGlobalMeshPose[MeshBoneIndex] = MeshLocalRefPose[MeshBoneIndex];
					}
				}

				for (int32 MeshBoneIndex = 0; MeshBoneIndex < NumMeshBones; ++MeshBoneIndex)
				{
					BankEntry.PositionKeys[KeyIndex] = FVector3f(SampledGlobalMeshPose[MeshBoneIndex].GetTranslation());
					BankEntry.RotationKeys[KeyIndex] = FQuat4f(SampledGlobalMeshPose[MeshBoneIndex].GetRotation());

					// Expand animated bounds
					{
						const VectorRegister4Float VecBonePosition = VectorLoadFloat3(&BankEntry.PositionKeys[KeyIndex].X);
						VecAnimatedBoundsMin = VectorMin(VecAnimatedBoundsMin, VecBonePosition);
						VecAnimatedBoundsMax = VectorMax(VecAnimatedBoundsMax, VecBonePosition);
					}

					++KeyIndex;
				}
			}
		}

		VectorStoreFloat3(VecAnimatedBoundsMin, &AnimatedBoundsMin);
		VectorStoreFloat3(VecAnimatedBoundsMax, &AnimatedBoundsMax);

		// Calculate (nearly) conservative bounds across all key frames
		// Also accounts for translated root motion
		BankEntry.SampledBounds = FBoxSphereBounds(
			FBox(
				FVector(AnimatedBoundsMin),
				FVector(AnimatedBoundsMax)
			)
		);

		// Apply per-sequence bounds scale (if specified)
		BankEntry.SampledBounds.BoxExtent *= BankSequence.BoundsScale;
		BankEntry.SampledBounds.SphereRadius *= BankSequence.BoundsScale;

		if (BankEntry.SampledBounds.ContainsNaN())
		{
			UE_LOGF(LogAnimBank, Error, "BankEntry contains NaN in sampled bounds!");
			return false;
		}

		check(BankEntry.PositionKeys.Num() > 0);
		check(BankEntry.RotationKeys.Num() > 0);
		check(BankEntry.FrameCount > 0);
	}

	if (Owner.IsCanceled())
	{
		return false;
	}

	return true;
}

void FAnimBankBuildAsyncCacheTask::Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority)
{
	if (BuildTask)
	{
		BuildTask->Reschedule(InThreadPool, InPriority);
	}
}

#endif

///////////////////////////////////////////////////////////////////////////////////////////////////

void UAnimBank::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::Animation);
	
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsFilterEditorOnly() && !Ar.IsObjectReferenceCollector() && !Ar.ShouldSkipBulkData())
	{
	#if WITH_EDITOR
		if (Ar.IsCooking())
		{
			if (IsCompiling())
			{
				FAnimBankCompilingManager::Get().FinishCompilation({this});
			}

			FAnimBankData& CookedData = CacheDerivedData(Ar.CookingTarget());
			Ar << CookedData;
		}
		else
	#endif
		{
			Ar << Data;
		}
	}
}

void UAnimBank::PostLoad()
{
	Super::PostLoad();

	if (FApp::CanEverRender())
	{
		// Only valid for cooked builds
		if (Data.Entries.Num() > 0)
		{
			InitResources();
		}
	#if WITH_EDITOR
		else if (ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform())
		{
			BeginCacheDerivedData(RunningPlatform);
		}
	#endif
	}

#if WITH_EDITOR
	OnDependenciesChanged.Broadcast(this);
#endif
}

void UAnimBank::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
}

void UAnimBank::BeginDestroy()
{
	ReleaseResources();

#if WITH_EDITOR
	OnDependenciesChanged.Broadcast(this);
#endif

	Super::BeginDestroy();
}

bool UAnimBank::IsReadyForFinishDestroy()
{
	if (!Super::IsReadyForFinishDestroy())
	{
		return false;
	}

#if WITH_EDITOR
	if (!TryCancelAsyncTasks())
	{
		return false;
	}
#endif

	return ReleaseResourcesFence.IsFenceComplete();
}

bool UAnimBank::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const
{
	return true;
}

void UAnimBank::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Data.Mapping.MeshGlobalRefPose.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Data.Mapping.PositionKeys.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Data.Mapping.RotationKeys.GetAllocatedSize());

	for (const FAnimBankEntry& Entry : Data.Entries)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(Entry));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Entry.PositionKeys.GetAllocatedSize());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Entry.RotationKeys.GetAllocatedSize());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Entry.ScalingKeys.GetAllocatedSize());
	}
}

void UAnimBank::InitResources()
{
	if (!FApp::CanEverRender())
	{
		return;
	}

	if (!bIsInitialized)
	{
		// TODO:
		//Data.ResourcesPtr->InitResources(this);
	}

	bIsInitialized = true;
}

void UAnimBank::ReleaseResources()
{
	if (!bIsInitialized)
	{
		return;
	}

	// TODO:
	//if (Data.ResourcesPtr->ReleaseResources())
	{
		// Make sure the renderer is done processing the command,
		// and done using the GPU resources before we overwrite the data.
		ReleaseResourcesFence.BeginFence();
	}

	bIsInitialized = false;
}

#if WITH_EDITOR

void UAnimBank::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!IsTemplate() && !FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		MarkPackageDirty();
	}

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimBank, Sequences))
		{
			// ...
		}
	}

	OnDependenciesChanged.Broadcast(this);

	// Synchronously build the new data. This calls InitResources.
	ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	BeginCacheDerivedData(RunningPlatform);
}

void UAnimBank::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	Super::BeginCacheForCookedPlatformData(TargetPlatform);
	BeginCacheDerivedData(TargetPlatform);
}

bool UAnimBank::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	const FIoHash KeyHash = CreateDerivedDataKeyHash(TargetPlatform);
	if (KeyHash.IsZero())
	{
		return true;
	}

	if (PollCacheDerivedData(KeyHash))
	{
		EndCacheDerivedData(KeyHash);
		return true;
	}

	return false;
}

void UAnimBank::ClearAllCachedCookedPlatformData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimBank::ClearAllCachedCookedPlatformData);

	if (!TryCancelAsyncTasks())
	{
		FinishAsyncTasks();
	}

	/**
	 * TryCancelAsyncTasks or FinishAsyncTasks should have been able to clear all tasks.If any tasks remain
	 * then they must still be running, and we would crash when attempting to delete them.
	*/
	check(CacheTasksByKeyHash.IsEmpty());

	DataByPlatformKeyHash.Empty();
	Super::ClearAllCachedCookedPlatformData();
}

bool UAnimBank::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive)
{
	UAnimationAsset::GetAllAnimationSequencesReferred(AnimationAssets, bRecursive);

	for (FAnimBankSequence& BankSequence : Sequences)
	{
		UAnimSequence* Sequence = BankSequence.Sequence;
		if (Sequence && !AnimationAssets.Contains(Sequence))
		{
			Sequence->HandleAnimReferenceCollection(AnimationAssets, bRecursive);
		}
	}

	return AnimationAssets.Num() > 0;
}

template<class AssetType>
inline void HandleAnimBankReferenceReplacement(AssetType*& OriginalAsset, const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	AssetType* CacheOriginalAsset = OriginalAsset;
	OriginalAsset = nullptr;

	if (UAnimationAsset* const* ReplacementAsset = ReplacementMap.Find(CacheOriginalAsset))
	{
		OriginalAsset = Cast<AssetType>(*ReplacementAsset);
	}
}

template<class AssetType>
inline void HandleAnimBankReferenceReplacement(TObjectPtr<AssetType>& OriginalAsset, const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	HandleAnimBankReferenceReplacement(static_cast<AssetType*&>(OriginalAsset), ReplacementMap);
}

void UAnimBank::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	UAnimationAsset::ReplaceReferredAnimations(ReplacementMap);

	for (FAnimBankSequence& BankSequence : Sequences)
	{
		HandleAnimBankReferenceReplacement(BankSequence.Sequence, ReplacementMap);
	}
}

bool UAnimBank::IsCompiling() const
{
	return CacheTasksByKeyHash.Num() > 0;
}

bool UAnimBank::TryCancelAsyncTasks()
{
	bool bHadCachedTaskForRunningPlatform = CacheTasksByKeyHash.Contains(DataKeyHash);
	
	for (auto It = CacheTasksByKeyHash.CreateIterator(); It; ++It)
	{
		if (It->Value->Poll())
		{
			It.RemoveCurrent();
		}
		else
		{
			It->Value->Cancel();

			// Try to see if we can remove the task now that it might have been canceled
			if (It->Value->Poll())
			{
				It.RemoveCurrent();
			}
		}
	}

	if (bHadCachedTaskForRunningPlatform && !CacheTasksByKeyHash.Contains(DataKeyHash))
	{
		// Reset the cached Key for the running platform since we won't have any GPU data
		DataKeyHash = FIoHash();
	}

	return CacheTasksByKeyHash.IsEmpty();
}

bool UAnimBank::IsAsyncTaskComplete() const
{
	for (auto& Pair : CacheTasksByKeyHash)
	{
		if (!Pair.Value->Poll())
		{
			return false;
		}
	}

	return true;
}

bool UAnimBank::WaitForAsyncTasks(float TimeLimitSeconds)
{
	double StartTimeSeconds = FPlatformTime::Seconds();
	for (auto& Pair : CacheTasksByKeyHash)
	{
		// Clamp to 0 as it implies polling
		const float TimeLimit = FMath::Min(0.0f, TimeLimitSeconds - (FPlatformTime::Seconds() - StartTimeSeconds));
		if (!Pair.Value->WaitWithTimeout(TimeLimit))
		{
			return false;
		}
	}

	return true;
}

void UAnimBank::FinishAsyncTasks()
{
	for (auto It = CacheTasksByKeyHash.CreateIterator(); It; ++It)
	{
		It->Value->Wait();
		It.RemoveCurrent();
	}
}

void UAnimBank::Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority)
{
	for (auto& Pair : CacheTasksByKeyHash)
	{
		Pair.Value->Reschedule(InThreadPool, InPriority);
	}
}

FIoHash UAnimBank::CreateDerivedDataKeyHash(const ITargetPlatform* TargetPlatform)
{
	FMemoryHasherBlake3 Writer;
	FGuid AnimBankVersionGuid(0x51842261, 0xDBAF4D8C, 0xB14ABA3D, 0xC9A914B5);
	Writer << AnimBankVersionGuid;

	if (IsValid(Asset))
	{
		FString AssetHash = Asset->BuildDerivedDataKey(TargetPlatform);
		Writer << AssetHash;
	}

	for (int32 SequenceIndex = 0; SequenceIndex < Sequences.Num(); ++SequenceIndex)
	{
		Writer << SequenceIndex;

		FAnimBankSequence& BankSequence = Sequences[SequenceIndex];
		if (IsValid(BankSequence.Sequence))
		{
			// Make sure all our required dependencies are loaded, we need them to compute the KeyHash
			FAnimationUtils::EnsureAnimSequenceLoaded(*BankSequence.Sequence);

			FIoHash SequenceHash = BankSequence.Sequence->GetDerivedDataKeyHash(TargetPlatform);
			Writer << SequenceHash;
		}

		uint32 Flags = BankFlagsFromSequence(BankSequence);
		Writer << Flags;

		Writer << BankSequence.Position;
		Writer << BankSequence.PlayRate;
		Writer << BankSequence.BoundsScale;
	}

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	FString ArmSuffix("_arm64");
	Writer << ArmSuffix;
#endif

	return Writer.Finalize();
}

FIoHash UAnimBank::BeginCacheDerivedData(const ITargetPlatform* TargetPlatform)
{
	const FIoHash KeyHash = CreateDerivedDataKeyHash(TargetPlatform);

	if (KeyHash.IsZero() || DataKeyHash == KeyHash || DataByPlatformKeyHash.Contains(KeyHash))
	{
		return KeyHash;
	}

	// If nothing has changed and we already started the compilation we should be all good.
	if (CacheTasksByKeyHash.Contains(KeyHash))
	{
		return KeyHash;
	}

	// Make sure we finish the previous build before starting another one
	FAnimBankCompilingManager::Get().FinishCompilation({ this });

	// Make sure the GPU is no longer referencing the current GPU resource data.
	ReleaseResources();
	ReleaseResourcesFence.Wait();
	Data = {};

	NotifyOnGPUDataChanged();

	FAnimBankData* TargetData = nullptr;
	if (TargetPlatform->IsRunningPlatform())
	{
		DataKeyHash = KeyHash;
		TargetData = &Data;
	}
	else
	{
		TargetData = DataByPlatformKeyHash.Emplace(KeyHash, MakeUnique<FAnimBankData>()).Get();
	}

	CacheTasksByKeyHash.Emplace(KeyHash, MakePimpl<FAnimBankBuildAsyncCacheTask>(KeyHash, TargetData, *this, TargetPlatform));

	// The compiling manager provides throttling, notification manager, etc... for the asset being built.
	FAnimBankCompilingManager::Get().AddAnimBanks({this});

	return KeyHash;
}

bool UAnimBank::PollCacheDerivedData(const FIoHash& KeyHash) const
{
	if (KeyHash.IsZero())
	{
		return true;
	}

	if (const TPimplPtr<FAnimBankBuildAsyncCacheTask>* Task = CacheTasksByKeyHash.Find(KeyHash))
	{
		return (*Task)->Poll();
	}

	return true;
}

void UAnimBank::EndCacheDerivedData(const FIoHash& KeyHash)
{
	if (KeyHash.IsZero())
	{
		return;
	}

	TPimplPtr<FAnimBankBuildAsyncCacheTask> Task;
	if (CacheTasksByKeyHash.RemoveAndCopyValue(KeyHash, Task))
	{
		Task->Wait();
	}
}

FAnimBankData& UAnimBank::CacheDerivedData(const ITargetPlatform* TargetPlatform)
{
	const FIoHash KeyHash = BeginCacheDerivedData(TargetPlatform);
	EndCacheDerivedData(KeyHash);
	FAnimBankData& AnimBankData = (DataKeyHash == KeyHash) ? Data : *DataByPlatformKeyHash[KeyHash];
	return AnimBankData;
}

FDelegateHandle UAnimBank::RegisterOnGPUDataChanged(const FOnRebuild& Delegate)
{
	return OnGPUDataChanged.Add(Delegate);
}

void UAnimBank::UnregisterOnGPUDataChanged(FDelegateUserObject Unregister)
{
	OnGPUDataChanged.RemoveAll(Unregister);
}

void UAnimBank::UnregisterOnGPUDataChanged(FDelegateHandle Handle)
{
	OnGPUDataChanged.Remove(Handle);
}

void UAnimBank::NotifyOnGPUDataChanged()
{
	OnGPUDataChanged.Broadcast();
}

#endif

float UAnimBank::GetSequencePlayLength(int32 SequenceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (IsCompiling())
	{
		if (Sequences.IsValidIndex(SequenceIndex))
		{
			if (Sequences[SequenceIndex].Sequence)
			{
				return Sequences[SequenceIndex].Sequence->GetPlayLength();
			}
		}
	}
	else
#endif
	if (Data.Entries.IsValidIndex(SequenceIndex))
	{
		const float SampleInterval = 1.0f / ANIM_BANK_SAMPLE_RATE;
		return static_cast<float>(Data.Entries[SequenceIndex].FrameCount - 1) * SampleInterval;
	}

	return 0.0f;
}

bool UAnimBank::IsSequenceLooping(int32 SequenceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (IsCompiling())
	{
		if (Sequences.IsValidIndex(SequenceIndex))
		{
			if (Sequences[SequenceIndex].Sequence)
			{
				return Sequences[SequenceIndex].bLooping;
			}
		}
	}
	else
#endif
	if (Data.Entries.IsValidIndex(SequenceIndex))
	{
		return Data.Entries[SequenceIndex].IsLooping();
	}

	return false;
}
///////////////////////////////////////////////////////////////////////////////////////////////////

static float GetAnimBankTrackLoopedPosition(float Position, float TrackLength)
{
	if (TrackLength >= 0.0f)
	{
		Position = TrackLength == 0.0f ? 0.0f : FMath::Fmod(Position, TrackLength);

		// Returns a negative range if input is negative.
		if (Position < 0.0f)
		{
			Position += TrackLength;
		}
	}
	return Position;
}

void FAnimBankTrackPackedData::Update(float SequenceLength, bool bLooping, float DeltaTime)
{
	if (bAutoPlay)
	{
		PreviousPosition = Position;
		Position += PlayRate * DeltaTime;
		bHasPreviousPosition = false;
		Wrap(SequenceLength, bLooping);
	}
}

void FAnimBankTrackPackedData::Wrap(float SequenceLength, bool bLooping)
{
	const EAnimSequenceTrackLoopMode LoopModeEnum = GetLoopMode();

	if (LoopModeEnum == EAnimSequenceTrackLoopMode::Loop)
	{
		Position         = GetAnimBankTrackLoopedPosition(Position, SequenceLength);
		PreviousPosition = GetAnimBankTrackLoopedPosition(PreviousPosition, SequenceLength);
	}
	else
	{
		Position         = FMath::Clamp(Position, 0.0f, SequenceLength);
		PreviousPosition = FMath::Clamp(PreviousPosition, 0.0f, SequenceLength);
	}
}

void FAnimBankTrackPackedData::Serialize(FArchive& Ar)
{
	constexpr auto MakeBitMask = [] (uint32 Bits) -> uint32 { return (Bits >= 32) ? 0xffffffffu : ((1u << Bits) - 1u); };

	constexpr uint32 ItemIndexBits = 20;
	constexpr uint32 ItemIndexMask = MakeBitMask(ItemIndexBits);

	constexpr uint32 LoopModeShift = ItemIndexBits;
	constexpr uint32 LoopModeBits  = 2;
	constexpr uint32 LoopModeMask  = MakeBitMask(LoopModeBits);

	constexpr uint32 AutoPlayShift = LoopModeShift + LoopModeBits;
	constexpr uint32 AutoPlayBits  = 1;
	constexpr uint32 AutoPlayMask  = MakeBitMask(AutoPlayBits);
	
	constexpr uint32 HasPreviousPositionShift = AutoPlayShift + AutoPlayBits;
	constexpr uint32 HasPreviousPositionBits  = 1;
	constexpr uint32 HasPreviousPositionMask  = MakeBitMask(HasPreviousPositionBits);

	uint32 PackedBits;
	if (Ar.IsSaving())
	{
		PackedBits =
			   (ItemIndex & ItemIndexMask)
			| ((LoopMode  & LoopModeMask ) << LoopModeShift)
			| ((bAutoPlay & AutoPlayMask ) << AutoPlayShift)
			| ((bHasPreviousPosition & HasPreviousPositionMask) << HasPreviousPositionShift);
		Ar << PackedBits;
		Ar << Position;

		if (bAutoPlay)
		{
			Ar << PlayRate;
		}
		else
		{
			Ar << PreviousPosition;
		}
	}
	else
	{
		Ar << PackedBits;
		ItemIndex =  PackedBits & ItemIndexMask;
		LoopMode  = (PackedBits >> LoopModeShift) & LoopModeMask;
		bAutoPlay = (PackedBits >> AutoPlayShift) & AutoPlayMask;
		bHasPreviousPosition = (PackedBits >> HasPreviousPositionShift) & HasPreviousPositionMask;
		Ar << Position;

		if (bAutoPlay)
		{
			Ar << PlayRate;
		}
		else
		{
			Ar << PreviousPosition;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FAnimBankRenderData::~FAnimBankRenderData()
{
	check(Proxies.Num() == 0);
}

void FAnimBankRenderData::InsertProxy(FInstancedSkinningSceneExtensionProxy* Proxy)
{
	Proxies.FindOrAdd(Proxy);
}

void FAnimBankRenderData::RemoveProxy(FInstancedSkinningSceneExtensionProxy* Proxy)
{
	Proxies.Remove(Proxy);
}

void FAnimBankRenderData::Patch(FAnimBankTrackPool::FPatch&& Patch)
{
	if (Patch.GetNumTracks() != Tracks.GetNumTracks())
	{
		for (FInstancedSkinningSceneExtensionProxy* Proxy : Proxies)
		{
			Proxy->SetUniqueAnimationCount(Patch.GetNumTracks());
		}
	}
	Tracks.Patch<FAnimBankTrackPackedData>(MoveTemp(Patch),
		[&](FAnimBankTrackPackedData& Dst, const FAnimBankTrackPackedData& Src, bool bWasActive, int32 Index)
	{
		Dst = Src;
	});
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FAnimBankTransformProviderProxy::FAnimBankTransformProviderProxy(TConstArrayView<FAnimBankItem> InAnimBankItems, FInstancedSkinningSceneExtensionProxy* InSceneExtensionProxy)
	: AnimBankItems(InAnimBankItems)
	, TrackAllocator(new FAnimBankTrackPool)
	, Tracks(TrackAllocator.Get())
	, SceneExtensionProxy(InSceneExtensionProxy)
{
	TrackAllocator->Reserve(AnimBankItems.Num());

	for (int32 ItemIndex = 0; ItemIndex < AnimBankItems.Num(); ++ItemIndex)
	{
		const FAnimBankItem& Item = AnimBankItems[ItemIndex];
		const FAnimBankData& BankData = Item.BankAsset->GetData();
		const FAnimBankEntry& BankEntry = BankData.Entries[Item.SequenceIndex];

		FAnimSequenceTrackAutoPlayData Data;
		Data.SequenceIndex        = ItemIndex;
		Data.Position             = BankEntry.Position;
		Data.PlayRate             = BankEntry.PlayRate;
		Data.LoopMode             = BankEntry.IsLooping() ? EAnimSequenceTrackLoopMode::Loop : EAnimSequenceTrackLoopMode::Clamp;

		TrackAllocator->AllocateTrack(FAnimBankTrackPackedData::Pack(Data));
	}
}

FAnimBankTransformProviderProxy::FAnimBankTransformProviderProxy(TConstArrayView<FAnimBankItem> InAnimBankItems, FInstancedSkinningSceneExtensionProxy* InSceneExtensionProxy, FAnimBankRenderData& InRenderData)
	: AnimBankItems(InAnimBankItems)
	, RenderData(&InRenderData)
	, Tracks(&InRenderData.Tracks)
	, SceneExtensionProxy(InSceneExtensionProxy)
{}

void FAnimBankTransformProviderProxy::CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList)
{
	if (RenderData)
	{
		RenderData->InsertProxy(SceneExtensionProxy);
	}

	AnimBankTransformProvider = Scene.GetAnimBankTransformProvider();

	if (AnimBankTransformProvider)
	{
		AnimBankTransformProvider->RegisterAnimBank(this);
	}
}

void FAnimBankTransformProviderProxy::DestroyRenderThreadResources()
{
	if (RenderData)
	{
		RenderData->RemoveProxy(SceneExtensionProxy);
	}

	if (AnimBankTransformProvider)
	{
		AnimBankTransformProvider->UnregisterAnimBank(this);
		AnimBankTransformProvider = nullptr;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

bool UAnimBankData::IsEnabled() const
{
	for (const FAnimBankItem& BankItem : AnimBankItems)
	{
		if (BankItem.BankAsset == nullptr)
		{
			return false;
		}

		if (BankItem.BankAsset->Asset == nullptr)
		{
			return false;
		}

	#if WITH_EDITORONLY_DATA
		if (BankItem.SequenceIndex >= BankItem.BankAsset->Sequences.Num())
		{
			return false;
		}
	#endif
	}

	return true;
}

const FGuid& UAnimBankData::GetTransformProviderID() const
{
	static FGuid AnimBankGPUProviderId(ANIM_BANK_GPU_TRANSFORM_PROVIDER_GUID);
	static FGuid AnimBankCPUProviderId(ANIM_BANK_CPU_TRANSFORM_PROVIDER_GUID);

	static const auto AnimBankGPUVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.AnimBank.GPU"));
	if (AnimBankGPUVar && AnimBankGPUVar->GetValueOnAnyThread() == 1)
	{
		return AnimBankGPUProviderId;
	}
	else
	{
		return AnimBankCPUProviderId;
	}
}

uint32 UAnimBankData::GetUniqueAnimationCount() const
{
	return AnimBankItems.Num();
}

bool UAnimBankData::UsesSkeletonBatching() const
{
	return false;
}

bool UAnimBankData::HasAnimationBounds() const
{
	return true;
}

bool UAnimBankData::GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const
{
	if (AnimationIndex < uint32(AnimBankItems.Num()))
	{
		const FAnimBankItem& BankItem = AnimBankItems[AnimationIndex];
		if (BankItem.BankAsset != nullptr)
		{
			const FAnimBankData& BankData = BankItem.BankAsset->GetData();
			if (BankItem.SequenceIndex < BankData.Entries.Num())
			{
				OutBounds = BankData.Entries[BankItem.SequenceIndex].SampledBounds;
				return true;
			}
		}
	}

	return false;
}

uint32 UAnimBankData::GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const
{
	const uint32 AnimationIndex = InstanceData.AnimationIndex;
	if (AnimationIndex < uint32(AnimBankItems.Num()))
	{
		return AnimationIndex * 2u;
	}
	
	return 0u;
}

bool UAnimBankData::IsAnimBankValid(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const
{
	for (const FAnimBankItem& Item : AnimBankItems)
	{
		if (Item.BankAsset == nullptr /* || Item.BankAsset->Asset != ExtensionProxy->GetSkinnedAsset() */)
		{
			return false;
		}

		const FAnimBankData& BankData = Item.BankAsset->GetData();

		if (Item.SequenceIndex >= BankData.Entries.Num())
		{
			return false;
		}
	}

	return true;
}

FTransformProviderRenderProxy* UAnimBankData::CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy) const
{
	if (IsAnimBankValid(SceneExtensionProxy))
	{
		return new FAnimBankTransformProviderProxy(AnimBankItems, SceneExtensionProxy);
	}
	return nullptr;
}

bool UAnimBankData::IsCompiling() const
{
	for (const FAnimBankItem& BankItem : AnimBankItems)
	{
		if (BankItem.BankAsset && BankItem.BankAsset->IsCompiling())
		{
			return true;
		}
	}

	return false;
}

#if WITH_EDITOR

void UAnimBankData::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	for (FAnimBankItem& BankItem : AnimBankItems)
	{
		if (BankItem.BankAsset)
		{
			BankItem.BankAsset->BeginCacheForCookedPlatformData(TargetPlatform);
		}
	}
}

bool UAnimBankData::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	return !IsCompiling();
}

#endif

float UAnimBankData::GetSequencePlayLength(int32 ItemIndex) const
{
	if (AnimBankItems.IsValidIndex(ItemIndex))
	{
		const FAnimBankItem& AnimBankItem = AnimBankItems[ItemIndex];

		if (const UAnimBank* BankAsset = AnimBankItem.BankAsset)
		{
			return BankAsset->GetSequencePlayLength(AnimBankItem.SequenceIndex);
		}
	}
	return 0.0f;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FAnimBankSequence::ValidatePosition()
{
	float Min = 0.0f, Max = 0.0f;

	if (Sequence)
	{
		Max = Sequence->GetPlayLength();
	}

	Position = FMath::Clamp<float>(Position, Min, Max);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::AnimBank
{

void ConvertLocalToGlobalSpaceTransforms(
	const FReferenceSkeleton& InRefSkeleton,
	const TArray<FTransform>& InLocalSpaceTransforms,
	TArray<FTransform>& OutGlobalSpaceTransforms
)
{
	OutGlobalSpaceTransforms.Reset();

	if (!ensure(InRefSkeleton.GetRawBoneNum() == InLocalSpaceTransforms.Num()))
	{
		return;
	}

	const int32 NumTransforms = InLocalSpaceTransforms.Num();
	OutGlobalSpaceTransforms.SetNumUninitialized(NumTransforms);

	for (int32 BoneIndex = 0; BoneIndex < NumTransforms; ++BoneIndex)
	{
		// Initialize to identity since some of them don't have tracks
		int32 IterBoneIndex = BoneIndex;
		FTransform ComponentSpaceTransform = InLocalSpaceTransforms[BoneIndex];

		do
		{
			const int32 ParentIndex = InRefSkeleton.GetRawParentIndex(IterBoneIndex); // Get only raw bones (no virtual)
			if (ParentIndex != INDEX_NONE)
			{
				ComponentSpaceTransform *= InLocalSpaceTransforms[ParentIndex];
			}

			IterBoneIndex = ParentIndex;
		}
		while (InRefSkeleton.IsValidIndex(IterBoneIndex));

		OutGlobalSpaceTransforms[BoneIndex] = ComponentSpaceTransform;
	}
}

void BuildSkinnedAssetMapping(const USkinnedAsset& Asset, FSkinnedAssetMapping& Mapping)
{
	const FReferenceSkeleton& MeshRefSkeleton = Asset.GetRefSkeleton();

	const int32 NumMeshBones = MeshRefSkeleton.GetRawBoneNum();

	// Get Number of RawBones (no virtual)
	Mapping.BoneCount = NumMeshBones;

	// Get the skeleton reference pose in local space;
	const TArray<FTransform>& MeshLocalRefPose = MeshRefSkeleton.GetRawRefBonePose();

	// Get the bone transforms in global pose.
	ConvertLocalToGlobalSpaceTransforms(MeshRefSkeleton, MeshRefSkeleton.GetRawRefBonePose(), Mapping.MeshGlobalRefPose);

	Mapping.PositionKeys.AddUninitialized(NumMeshBones);
	Mapping.RotationKeys.AddUninitialized(NumMeshBones);
	for (int32 MeshBoneIndex = 0; MeshBoneIndex < NumMeshBones; ++MeshBoneIndex)
	{
		const FTransform InvMeshGlobalRefPoseXform = Mapping.MeshGlobalRefPose[MeshBoneIndex].Inverse();
		Mapping.PositionKeys[MeshBoneIndex] = FVector3f(InvMeshGlobalRefPoseXform.GetTranslation());
		Mapping.RotationKeys[MeshBoneIndex] = FQuat4f(InvMeshGlobalRefPoseXform.GetRotation());
	}
}

} // UE::AnimBank

FAnimBankItem::FAnimBankItem() = default;

FAnimBankItem::FAnimBankItem(const FAnimBankItem& InBankItem)
{
	BankAsset		= InBankItem.BankAsset;
	SequenceIndex	= InBankItem.SequenceIndex;
}

FAnimBankItem::FAnimBankItem(const FSoftAnimBankItem& InBankItem)
{
	BankAsset		= InBankItem.BankAsset.LoadSynchronous();
	SequenceIndex	= InBankItem.SequenceIndex;
}

bool FAnimBankItem::operator!=(const FAnimBankItem& Other) const
{
	return !(*this == Other);
}

bool FAnimBankItem::operator==(const FAnimBankItem& Other) const
{
	return BankAsset == Other.BankAsset && SequenceIndex == Other.SequenceIndex;
}

FSoftAnimBankItem::FSoftAnimBankItem() = default;

FSoftAnimBankItem::FSoftAnimBankItem(const FSoftAnimBankItem& InBankItem)
{
	BankAsset		= InBankItem.BankAsset.Get();
	SequenceIndex	= InBankItem.SequenceIndex;
}

FSoftAnimBankItem::FSoftAnimBankItem(const FAnimBankItem& InBankItem)
{
	BankAsset		= InBankItem.BankAsset.Get();
	SequenceIndex	= InBankItem.SequenceIndex;
}

bool FSoftAnimBankItem::operator!=(const FSoftAnimBankItem& Other) const
{
	return !(*this == Other);
}

bool FSoftAnimBankItem::operator==(const FSoftAnimBankItem& Other) const
{
	return BankAsset == Other.BankAsset && SequenceIndex == Other.SequenceIndex;
}

FSkinnedMeshComponentDescriptorBase::FSkinnedMeshComponentDescriptorBase()
{
	// Note: should not really be used - prefer using FSkinnedMeshComponentDescriptor or FSoftSkinnedMeshComponentDescriptor
	InitFrom(UInstancedSkinnedMeshComponent::StaticClass()->GetDefaultObject<UInstancedSkinnedMeshComponent>());
}

FSkinnedMeshComponentDescriptorBase::FSkinnedMeshComponentDescriptorBase(ENoInit) {}
FSkinnedMeshComponentDescriptorBase::FSkinnedMeshComponentDescriptorBase(const FSkinnedMeshComponentDescriptorBase& InDescriptor) = default;
FSkinnedMeshComponentDescriptorBase::~FSkinnedMeshComponentDescriptorBase() = default;

UInstancedSkinnedMeshComponent* FSkinnedMeshComponentDescriptorBase::CreateComponent(UObject* Outer, FName Name, EObjectFlags ObjectFlags) const
{
	UInstancedSkinnedMeshComponent* ISKMComponent = NewObject<UInstancedSkinnedMeshComponent>(Outer, ComponentClass, Name, ObjectFlags);

	InitComponent(ISKMComponent);

	return ISKMComponent;
}

void FSkinnedMeshComponentDescriptorBase::InitFrom(const UInstancedSkinnedMeshComponent* Template, bool bInitBodyInstance)
{
	check(Template);

	Mobility = Template->Mobility;
	InstanceMinDrawDistance = Template->InstanceMinDrawDistance;
	Template->GetCullDistances(InstanceStartCullDistance, InstanceEndCullDistance);
	ComponentClass = Template->GetClass();
	bCastShadow = Template->CastShadow;
	bCastDynamicShadow = Template->bCastDynamicShadow;
	bCastStaticShadow = Template->bCastStaticShadow;
	bCastVolumetricTranslucentShadow = Template->bCastVolumetricTranslucentShadow;
	bCastContactShadow = Template->bCastContactShadow;
	bSelfShadowOnly = Template->bSelfShadowOnly;
	bCastFarShadow = Template->bCastFarShadow;
	bCastInsetShadow = Template->bCastInsetShadow;
	bCastCinematicShadow = Template->bCastCinematicShadow;
	bCastHiddenShadow = Template->bCastHiddenShadow;
	bCastShadowAsTwoSided = Template->bCastShadowAsTwoSided;
	bVisibleInRayTracing = Template->bVisibleInRayTracing;
	bVisibleInReflections = Template->bVisibleInReflections;
	bAffectDynamicIndirectLighting = Template->bAffectDynamicIndirectLighting;
	bAffectDistanceFieldLighting = Template->bAffectDistanceFieldLighting;
	bVisible = Template->GetVisibleFlag();
	bRenderStatic = Template->bRenderStatic;
	bEvaluateWorldPositionOffset = Template->bEvaluateWorldPositionOffset;
	WorldPositionOffsetDisableDistance = Template->WorldPositionOffsetDisableDistance;
	NanitePixelProgrammableDistance = Template->NanitePixelProgrammableDistance;
	DetailMode = Template->DetailMode;
	PrimitiveBoundsOverride = Template->GetPrimitiveBoundsOverride();
#if WITH_EDITOR
	HLODBatchingPolicy = Template->HLODBatchingPolicy;
	bIncludeInHLOD = Template->bEnableAutoLODGeneration;
#endif

	bIsInstanceDataGPUOnly = Template->UsesGPUOnlyInstances();
	if (bIsInstanceDataGPUOnly)
	{
		NumInstancesGPUOnly = Template->GetInstanceCountGPUOnly();
		NumCustomDataFloatsGPUOnly = Template->GetNumCustomDataFloats();
	}

	if (bInitBodyInstance)
	{
		// If we are building this batch live in the editor, then the Body Properties will be in a different runtime state.
		if (Template->BodyInstance.BodySetup.Get() == nullptr)
		{
			BodyInstance.CopyBodyInstancePropertiesFrom(&Template->BodyInstance);
		}
		else
		{
			BodyInstance.CopyRuntimeBodyInstancePropertiesFrom(&Template->BodyInstance);
		}
	}
}

void FSkinnedMeshComponentDescriptorBase::InitComponent(UInstancedSkinnedMeshComponent* Component) const
{
	Component->Mobility = Mobility;
	Component->InstanceMinDrawDistance = InstanceMinDrawDistance;
	Component->SetCullDistances(InstanceStartCullDistance, InstanceEndCullDistance);
	Component->CastShadow = bCastShadow;
	Component->bCastDynamicShadow = bCastDynamicShadow;
	Component->bCastStaticShadow = bCastStaticShadow;
	Component->bCastVolumetricTranslucentShadow = bCastVolumetricTranslucentShadow;
	Component->bCastContactShadow = bCastContactShadow;
	Component->bSelfShadowOnly = bSelfShadowOnly;
	Component->bCastFarShadow = bCastFarShadow;
	Component->bCastInsetShadow = bCastInsetShadow;
	Component->bCastCinematicShadow = bCastCinematicShadow;
	Component->bCastHiddenShadow = bCastHiddenShadow;
	Component->bCastShadowAsTwoSided = bCastShadowAsTwoSided;
	Component->bVisibleInRayTracing = bVisibleInRayTracing;
	Component->bVisibleInReflections = bVisibleInReflections;
	Component->bAffectDynamicIndirectLighting = bAffectDynamicIndirectLighting;
	Component->bAffectDistanceFieldLighting = bAffectDistanceFieldLighting;
	Component->SetPrimitiveBoundsOverride(PrimitiveBoundsOverride);
	Component->SetVisibleFlag(bVisible);
	Component->bRenderStatic = bRenderStatic;
	Component->bEvaluateWorldPositionOffset = bEvaluateWorldPositionOffset;
	Component->BodyInstance.CopyBodyInstancePropertiesFrom(&BodyInstance);
	Component->WorldPositionOffsetDisableDistance = WorldPositionOffsetDisableDistance;
	Component->NanitePixelProgrammableDistance = NanitePixelProgrammableDistance;
	Component->DetailMode = DetailMode;
#if WITH_EDITOR
	Component->HLODBatchingPolicy = HLODBatchingPolicy;
	Component->bEnableAutoLODGeneration = bIncludeInHLOD;
#endif

	Component->SetInstanceDataGPUOnly(bIsInstanceDataGPUOnly);
	if (bIsInstanceDataGPUOnly)
	{
		Component->SetNumGPUInstances(NumInstancesGPUOnly);
		Component->SetNumCustomDataFloats(NumCustomDataFloatsGPUOnly);
	}
}

bool FSkinnedMeshComponentDescriptorBase::operator!=(const FSkinnedMeshComponentDescriptorBase& Other) const
{
	return !(*this == Other);
}

bool FSkinnedMeshComponentDescriptorBase::operator==(const FSkinnedMeshComponentDescriptorBase& Other) const
{
	return
		ComponentClass == Other.ComponentClass &&
		Mobility == Other.Mobility &&
		InstanceMinDrawDistance == Other.InstanceMinDrawDistance &&
		InstanceStartCullDistance == Other.InstanceStartCullDistance &&
		InstanceEndCullDistance == Other.InstanceEndCullDistance &&
#if WITH_EDITOR
		HLODBatchingPolicy == Other.HLODBatchingPolicy &&
		bIncludeInHLOD == Other.bIncludeInHLOD &&
#endif 
		bCastShadow == Other.bCastShadow &&
		bAffectDynamicIndirectLighting == Other.bAffectDynamicIndirectLighting &&
		bAffectDistanceFieldLighting == Other.bAffectDistanceFieldLighting &&
		bCastDynamicShadow == Other.bCastDynamicShadow &&
		bCastStaticShadow == Other.bCastStaticShadow &&
		bCastVolumetricTranslucentShadow == Other.bCastVolumetricTranslucentShadow &&
		bCastContactShadow == Other.bCastContactShadow &&
		bSelfShadowOnly == Other.bSelfShadowOnly &&
		bCastFarShadow == Other.bCastFarShadow &&
		bCastInsetShadow == Other.bCastInsetShadow &&
		bCastCinematicShadow == Other.bCastCinematicShadow &&
		bCastHiddenShadow == Other.bCastHiddenShadow &&
		bCastShadowAsTwoSided == Other.bCastShadowAsTwoSided &&
		bIsInstanceDataGPUOnly == Other.bIsInstanceDataGPUOnly &&
		PrimitiveBoundsOverride == Other.PrimitiveBoundsOverride &&
		NumInstancesGPUOnly == Other.NumInstancesGPUOnly &&
		NumCustomDataFloatsGPUOnly == Other.NumCustomDataFloatsGPUOnly &&
		bVisible == Other.bVisible &&
		bVisibleInRayTracing == Other.bVisibleInRayTracing &&
		bVisibleInReflections == Other.bVisibleInReflections &&
		bEvaluateWorldPositionOffset == Other.bEvaluateWorldPositionOffset &&
		WorldPositionOffsetDisableDistance == Other.WorldPositionOffsetDisableDistance &&
		NanitePixelProgrammableDistance == Other.NanitePixelProgrammableDistance &&
		DetailMode == Other.DetailMode &&
		BodyInstance.GetCollisionEnabled() == Other.BodyInstance.GetCollisionEnabled() &&
		BodyInstance.GetCollisionResponse() == Other.BodyInstance.GetCollisionResponse() &&
		BodyInstance.DoesUseCollisionProfile() == Other.BodyInstance.DoesUseCollisionProfile() &&
		(!BodyInstance.DoesUseCollisionProfile() || (BodyInstance.GetCollisionProfileName() == Other.BodyInstance.GetCollisionProfileName()));
}

FSkinnedMeshComponentDescriptor::FSkinnedMeshComponentDescriptor()
	: FSkinnedMeshComponentDescriptorBase(NoInit)
{
	// Make sure we have proper defaults
	InitFrom(UInstancedSkinnedMeshComponent::StaticClass()->GetDefaultObject<UInstancedSkinnedMeshComponent>());
}

FSkinnedMeshComponentDescriptor::FSkinnedMeshComponentDescriptor(ENoInit) {}
FSkinnedMeshComponentDescriptor::FSkinnedMeshComponentDescriptor(const FSkinnedMeshComponentDescriptor& InDescriptor) = default;
FSkinnedMeshComponentDescriptor::~FSkinnedMeshComponentDescriptor() = default;

FSkinnedMeshComponentDescriptor::FSkinnedMeshComponentDescriptor(const FSoftSkinnedMeshComponentDescriptor& InDescriptor)
	: FSkinnedMeshComponentDescriptorBase(InDescriptor)
{
	SkinnedAsset = InDescriptor.SkinnedAsset.LoadSynchronous();
	TransformProvider = InDescriptor.TransformProvider.LoadSynchronous();
	// AB-TODO: Algo::Transform(InDescriptor.OverrideMaterials, OverrideMaterials, [](TSoftObjectPtr<UMaterialInterface> Material) { return Material.LoadSynchronous(); });
	// AB-TODO: OverlayMaterial = InDescriptor.OverlayMaterial.LoadSynchronous();
	// AB-TODO: Algo::Transform(InDescriptor.RuntimeVirtualTextures, RuntimeVirtualTextures, [](TSoftObjectPtr<URuntimeVirtualTexture> RVT) { return RVT.LoadSynchronous(); });
	Hash = InDescriptor.Hash;
}

UInstancedSkinnedMeshComponent* FSkinnedMeshComponentDescriptor::CreateComponent(UObject* Outer, FName Name, EObjectFlags ObjectFlags) const
{
	UInstancedSkinnedMeshComponent* Component = NewObject<UInstancedSkinnedMeshComponent>(Outer, ComponentClass, Name, ObjectFlags);
	InitComponent(Component);
	return Component;
}

void FSkinnedMeshComponentDescriptor::InitFrom(const UInstancedSkinnedMeshComponent* Template, bool bInitBodyInstance)
{
	Super::InitFrom(Template, bInitBodyInstance);

	SkinnedAsset = Template->GetSkinnedAsset();
	TransformProvider = Template->GetTransformProvider();
}

uint32 FSkinnedMeshComponentDescriptor::ComputeHash() const
{
	FSkinnedMeshComponentDescriptor& MutableSelf = *const_cast<FSkinnedMeshComponentDescriptor*>(this);
	Hash = 0; // we don't want the hash to impact the calculation
	
	FArchiveCrc32 CrcArchive;
	CrcArchive << *this;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

void FSkinnedMeshComponentDescriptor::InitComponent(UInstancedSkinnedMeshComponent* Component) const
{
	Super::InitComponent(Component);
	Component->SetSkinnedAsset(SkinnedAsset);
	Component->SetTransformProvider(TransformProvider);
}

void FSkinnedMeshComponentDescriptor::PostLoadFixup(UObject* Loader)
{
	check(Loader);
}

bool FSkinnedMeshComponentDescriptor::operator!=(const FSkinnedMeshComponentDescriptor& Other) const
{
	return !(*this == Other);
}

bool FSkinnedMeshComponentDescriptor::operator==(const FSkinnedMeshComponentDescriptor& Other) const
{
	return
		(Hash == 0 || Other.Hash == 0 || Hash == Other.Hash) && // Check hash first, other checks are in case of Hash collision
		SkinnedAsset == Other.SkinnedAsset &&
		TransformProvider == Other.TransformProvider &&
		Super::operator==(Other);
}

FSoftSkinnedMeshComponentDescriptor::FSoftSkinnedMeshComponentDescriptor()
	: FSkinnedMeshComponentDescriptorBase(NoInit)
{
	// Make sure we have proper defaults
	InitFrom(UInstancedSkinnedMeshComponent::StaticClass()->GetDefaultObject<UInstancedSkinnedMeshComponent>());
}

FSoftSkinnedMeshComponentDescriptor::FSoftSkinnedMeshComponentDescriptor(ENoInit) {}
FSoftSkinnedMeshComponentDescriptor::FSoftSkinnedMeshComponentDescriptor(const FSoftSkinnedMeshComponentDescriptor& InDescriptor) = default;
FSoftSkinnedMeshComponentDescriptor::~FSoftSkinnedMeshComponentDescriptor() = default;

FSoftSkinnedMeshComponentDescriptor::FSoftSkinnedMeshComponentDescriptor(const FSkinnedMeshComponentDescriptor& InDescriptor)
	: FSkinnedMeshComponentDescriptorBase(InDescriptor)
{
	SkinnedAsset = InDescriptor.SkinnedAsset;
	TransformProvider = InDescriptor.TransformProvider;
	// AB-TODO: Algo::Transform(InDescriptor.OverrideMaterials, OverrideMaterials, [](TObjectPtr<UMaterialInterface> Material) { return Material; });
	// AB-TODO: OverlayMaterial = InDescriptor.OverlayMaterial;
	// AB-TODO: Algo::Transform(InDescriptor.RuntimeVirtualTextures, RuntimeVirtualTextures, [](TObjectPtr<URuntimeVirtualTexture> RVT) { return RVT; });
	Hash = InDescriptor.Hash;
}

UInstancedSkinnedMeshComponent* FSoftSkinnedMeshComponentDescriptor::CreateComponent(UObject* Outer, FName Name, EObjectFlags ObjectFlags) const
{
	UInstancedSkinnedMeshComponent* Component = NewObject<UInstancedSkinnedMeshComponent>(Outer, ComponentClass, Name, ObjectFlags);
	InitComponent(Component);
	return Component;
}

void FSoftSkinnedMeshComponentDescriptor::InitFrom(const UInstancedSkinnedMeshComponent* Template, bool bInitBodyInstance)
{
	Super::InitFrom(Template, bInitBodyInstance);
	SkinnedAsset = Template->GetSkinnedAsset();
	TransformProvider = Template->GetTransformProvider();
}

uint32 FSoftSkinnedMeshComponentDescriptor::ComputeHash() const
{
	Hash = 0; // we don't want the hash to impact the calculation
	
	FArchiveCrc32 CrcArchive;
	CrcArchive << *this;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

void FSoftSkinnedMeshComponentDescriptor::InitComponent(UInstancedSkinnedMeshComponent* Component) const
{
	Super::InitComponent(Component);

	Component->SetSkinnedAsset(SkinnedAsset.LoadSynchronous());
	Component->SetTransformProvider(TransformProvider.LoadSynchronous());
}

void FSoftSkinnedMeshComponentDescriptor::PostLoadFixup(UObject* Loader)
{
	check(Loader);
}

bool FSoftSkinnedMeshComponentDescriptor::operator!=(const FSoftSkinnedMeshComponentDescriptor& Other) const
{
	return !(*this == Other);
}

bool FSoftSkinnedMeshComponentDescriptor::operator==(const FSoftSkinnedMeshComponentDescriptor& Other) const
{
	return
		(Hash == 0 || Other.Hash == 0 || Hash == Other.Hash) && // Check hash first, other checks are in case of Hash collision
		SkinnedAsset == Other.SkinnedAsset &&
		TransformProvider == Other.TransformProvider &&
		Super::operator==(Other);
}

#undef LOCTEXT_NAMESPACE

