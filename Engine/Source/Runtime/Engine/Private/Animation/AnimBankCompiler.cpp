// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBankCompiler.h"
#include "UObject/Package.h"

#if WITH_EDITOR

#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"
#include "Algo/NoneOf.h"
#include "Misc/QueuedThreadPool.h"
#include "ObjectCacheContext.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "Misc/App.h"
#include "Animation/AnimBank.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "AnimationSequenceCompiler.h"
#include "Components/InstancedSkinnedMeshComponent.h"

#define LOCTEXT_NAMESPACE "AnimBankCompiler"

static AsyncCompilationHelpers::FAsyncCompilationStandardCVars CVarAsyncAnimBankStandard(
	TEXT("AnimBank"),
	TEXT("animation banks"),
	FConsoleCommandDelegate::CreateLambda(
	[]()
	{
		FAnimBankCompilingManager::Get().FinishAllCompilation();
	}
));

namespace AnimBankCompilingManagerImpl
{
	static void EnsureInitializedCVars()
	{
		static bool bIsInitialized = false;

		if (!bIsInitialized)
		{
			bIsInitialized = true;

			AsyncCompilationHelpers::EnsureInitializedCVars(
				TEXT("AnimBank"),
				CVarAsyncAnimBankStandard.AsyncCompilation,
				CVarAsyncAnimBankStandard.AsyncCompilationMaxConcurrency);
		}
	}
}

FAnimBankCompilingManager::FAnimBankCompilingManager()
: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	AnimBankCompilingManagerImpl::EnsureInitializedCVars();
	PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddRaw(this, &FAnimBankCompilingManager::OnPostReachabilityAnalysis);
}

void FAnimBankCompilingManager::OnPostReachabilityAnalysis()
{
	if (GetNumRemainingAssets())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankCompilingManager::CancelUnreachableMeshes);

		TArray<UAnimBank*> PendingAnimBanks;
		PendingAnimBanks.Reserve(GetNumRemainingAssets());

		for (auto Iterator = RegisteredAnimBanks.CreateIterator(); Iterator; ++Iterator)
		{
			UAnimBank* AnimBank = Iterator->GetEvenIfUnreachable();
			if (AnimBank && AnimBank->IsUnreachable())
			{
				UE_LOGF(LogAnimBank, Verbose, "Cancelling animation bank %ls compilation because it's being garbage collected", *AnimBank->GetName());

				if (AnimBank->TryCancelAsyncTasks())
				{
					Iterator.RemoveCurrent();
				}
				else
				{
					PendingAnimBanks.Add(AnimBank);
				}
			}
		}

		FinishCompilation(PendingAnimBanks);
	}
}

FName FAnimBankCompilingManager::GetAssetTypeName() const
{
	return TEXT("UE-AnimBank");
}

FTextFormat FAnimBankCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("AnimBankNameFormat", "{0}|plural(one=Animation Bank,other=Animation Banks)");
}

TArrayView<FName> FAnimBankCompilingManager::GetDependentTypeNames() const
{
	static FName DependentTypeNames[] =	
	{ 
		// AnimBank can wait on AnimSequence to finish their own compilation before compiling itself
		// so they need to be processed before us. This is especially important when FinishAllCompilation is issued
		// so that we know once we're called that all anim sequences have finished compiling.
		UE::Anim::FAnimSequenceCompilingManager::GetStaticAssetTypeName() 
	};
	return TArrayView<FName>(DependentTypeNames);
}

EQueuedWorkPriority FAnimBankCompilingManager::GetBasePriority(UAnimBank* InAnimBank) const
{
	return EQueuedWorkPriority::Low;
}

FQueuedThreadPool* FAnimBankCompilingManager::GetThreadPool() const
{
	static FQueuedThreadPoolDynamicWrapper* GAnimBankThreadPool = nullptr;
	if (GAnimBankThreadPool == nullptr && FAssetCompilingManager::Get().GetThreadPool() != nullptr)
	{
		// Animation banks will be scheduled on the asset thread pool, where concurrency limits might by dynamically adjusted depending on memory constraints.
		GAnimBankThreadPool = new FQueuedThreadPoolDynamicWrapper(FAssetCompilingManager::Get().GetThreadPool(), -1, [](EQueuedWorkPriority) { return EQueuedWorkPriority::Low; });

		AsyncCompilationHelpers::BindThreadPoolToCVar(
			GAnimBankThreadPool,
			CVarAsyncAnimBankStandard.AsyncCompilation,
			CVarAsyncAnimBankStandard.AsyncCompilationResume,
			CVarAsyncAnimBankStandard.AsyncCompilationMaxConcurrency
		);
	}

	return GAnimBankThreadPool;
}

void FAnimBankCompilingManager::Shutdown()
{
	bHasShutdown = true;
	if (GetNumRemainingAssets())
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankCompilingManager::Shutdown);

		TArray<UAnimBank*> PendingAnimBanks;
		PendingAnimBanks.Reserve(GetNumRemainingAssets());

		for (TWeakObjectPtr<UAnimBank>& WeakAnimBank : RegisteredAnimBanks)
		{
			if (WeakAnimBank.IsValid())
			{
				UAnimBank* AnimBank = WeakAnimBank.Get();
				if (!AnimBank->TryCancelAsyncTasks())
				{
					PendingAnimBanks.Add(AnimBank);
				}
			}
		}

		FinishCompilation(PendingAnimBanks);
	}

	FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostReachabilityAnalysisHandle);
}

TRACE_DECLARE_INT_COUNTER(QueuedAnimBankCompilation, TEXT("AsyncCompilation/QueuedAnimBank"));
void FAnimBankCompilingManager::UpdateCompilationNotification()
{
	TRACE_COUNTER_SET(QueuedAnimBankCompilation, GetNumRemainingAssets());
	Notification->Update(GetNumRemainingAssets());
}

void FAnimBankCompilingManager::PostCompilation(TArrayView<UAnimBank* const> InAnimBanks)
{
	if (InAnimBanks.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OnAssetPostCompileEvent);

		TArray<FAssetCompileData> AssetsData;
		AssetsData.Reserve(InAnimBanks.Num());

		for (UAnimBank* AnimBank : InAnimBanks)
		{
			// Do not broadcast an event for unreachable objects
			if (!AnimBank->IsUnreachable())
			{
				AssetsData.Emplace(AnimBank);

				if (FApp::CanEverRender())
				{
					AnimBank->InitResources();
					AnimBank->NotifyOnGPUDataChanged();
				}
			}
		}

		if (AssetsData.Num())
		{
			FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast(AssetsData);
		}
	}
}

template <typename LambdaType>
void ForEachInstancedSkinnedMeshComponent(LambdaType&& Lambda, TConstArrayView<UAnimBank* const> AnimBanks)
{
	FObjectCacheContextScope ObjectCacheScope;

	for (UInstancedSkinnedMeshComponent* Component : ObjectCacheScope.GetContext().GetInstancedSkinnedMeshComponents())
	{
		UTransformProviderData* ProviderData = Component->GetTransformProvider();
		if (UAnimBankData* AnimBankData = Cast<UAnimBankData>(ProviderData))
		{
			bool bMatched = false;
			for (const FAnimBankItem& BankItem : AnimBankData->AnimBankItems)
			{
				for (const UAnimBank* AnimBank : AnimBanks)
				{
					if (BankItem.BankAsset == AnimBank)
					{
						bMatched = true;
						break;
					}
				}

				if (bMatched)
				{
					break;
				}
			}

			if (bMatched)
			{
				Lambda(Component);
			}
		}
	}
}

void FAnimBankCompilingManager::PostCompilation(UAnimBank* AnimBank)
{
	using namespace AnimBankCompilingManagerImpl;

	// If AsyncTask is null here, the task got canceled so we don't need to do anything
	if (!IsEngineExitRequested())
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(PostCompilation);

		AnimBank->FinishAsyncTasks();

		// Do not do anything else if the AnimBank is being garbage collected
		if (AnimBank->IsUnreachable())
		{
			return;
		}

		AnimBank->InitResources();

		ForEachInstancedSkinnedMeshComponent([] (UInstancedSkinnedMeshComponent* Component)
		{
			Component->PostAssetCompilation();

		}, MakeArrayView({ AnimBank }));

		// Calling this delegate during app exit might be quite dangerous and lead to crash
		// if the content browser wants to refresh a thumbnail it might try to load a package
		// which will then fail due to various reasons related to the editor shutting down.
		// Triggering this callback while garbage collecting can also result in listeners trying to look up objects
		if (!GExitPurge && !IsGarbageCollecting())
		{
			// Generate an empty property changed event, to force the asset registry tag
			// to be refreshed now that RenderData is available.
			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(AnimBank, EmptyPropertyChangedEvent);
		}
	}
}

FAnimBankCompilingManager& FAnimBankCompilingManager::Get()
{
	static FAnimBankCompilingManager Singleton;
	return Singleton;
}

int32 FAnimBankCompilingManager::GetNumRemainingAssets() const
{
	return RegisteredAnimBanks.Num();
}

void FAnimBankCompilingManager::AddAnimBanks(TArrayView<UAnimBank* const> InAnimBanks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankCompilingManager::AddAnimBanks);
	check(IsInGameThread());

	for (UAnimBank* AnimBank : InAnimBanks)
	{
		RegisteredAnimBanks.Emplace(AnimBank);
	}

	ForEachInstancedSkinnedMeshComponent([] (UInstancedSkinnedMeshComponent* Component)
	{
		Component->PreAssetCompilation();

	}, InAnimBanks);

	TRACE_COUNTER_SET(QueuedAnimBankCompilation, GetNumRemainingAssets());
}

void FAnimBankCompilingManager::FinishCompilation(TArrayView<UAnimBank* const> InAnimBanks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankCompilingManager::FinishCompilation);

	// Allow calls from any thread if the meshes are already finished compiling.
	if (Algo::NoneOf(InAnimBanks, &UAnimBank::IsCompiling))
	{
		return;
	}

	check(IsInGameThread());

	TArray<UAnimBank*> PendingAnimBanks;
	PendingAnimBanks.Reserve(InAnimBanks.Num());

	int32 AnimBankIndex = 0;
	for (UAnimBank* AnimBank : InAnimBanks)
	{
		if (RegisteredAnimBanks.Contains(AnimBank))
		{
			PendingAnimBanks.Emplace(AnimBank);
		}
	}

	if (PendingAnimBanks.Num())
	{
		class FCompilableAnimBank : public AsyncCompilationHelpers::ICompilable
		{
		public:
			FCompilableAnimBank(UAnimBank* InAnimBank)
			: AnimBank(InAnimBank)
			{
			}

			void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority)
			{
				AnimBank->Reschedule(InThreadPool, InPriority);
			}

			bool WaitCompletionWithTimeout(float TimeLimitSeconds) override
			{
				return AnimBank->WaitForAsyncTasks(TimeLimitSeconds);
			}

			UAnimBank* AnimBank;
			FName GetName() override { return AnimBank->GetOutermost()->GetFName(); }
		};

		TArray<FCompilableAnimBank> CompilableAnimBanks(PendingAnimBanks);
		FObjectCacheContextScope ObjectCacheScope;
		AsyncCompilationHelpers::FinishCompilation(
			[&CompilableAnimBanks](int32 Index)	-> AsyncCompilationHelpers::ICompilable& { return CompilableAnimBanks[Index]; },
			CompilableAnimBanks.Num(),
			LOCTEXT("AnimBanks", "Animation Banks"),
			LogAnimBank,
			[this](AsyncCompilationHelpers::ICompilable* Object)
			{
				UAnimBank* AnimBank = static_cast<FCompilableAnimBank*>(Object)->AnimBank;
				PostCompilation(AnimBank);
				RegisteredAnimBanks.Remove(AnimBank);
			}
		);

		PostCompilation(PendingAnimBanks);
	}
}

void FAnimBankCompilingManager::FinishCompilationsForGame()
{
	// Nothing special to do when we PIE for now.
}

void FAnimBankCompilingManager::FinishAllCompilation()
{
	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankCompilingManager::FinishAllCompilation)

	if (GetNumRemainingAssets())
	{
		TArray<UAnimBank*> PendingAnimBanks;
		PendingAnimBanks.Reserve(GetNumRemainingAssets());

		for (TWeakObjectPtr<UAnimBank>& AnimBank : RegisteredAnimBanks)
		{
			if (AnimBank.IsValid())
			{
				PendingAnimBanks.Add(AnimBank.Get());
			}
		}

		FinishCompilation(PendingAnimBanks);
	}
}

void FAnimBankCompilingManager::Reschedule()
{
	// TODO Prioritize animation banks that are nearest to the viewport
}

void FAnimBankCompilingManager::ProcessAnimBanks(bool bLimitExecutionTime, int32 MinBatchSize)
{
	using namespace AnimBankCompilingManagerImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankCompilingManager::ProcessAnimBanks);

	const int32 NumRemainingBanks = GetNumRemainingAssets();
	
	// Spread out the load over multiple frames but if too many banks, convergence is more important than frame time
	const int32 MaxBankUpdatesPerFrame = bLimitExecutionTime ? FMath::Max(64, NumRemainingBanks / 10) : INT32_MAX;

	FObjectCacheContextScope ObjectCacheScope;
	if (NumRemainingBanks && NumRemainingBanks >= MinBatchSize)
	{
		TSet<UAnimBank*> AnimBanksToProcess;
		for (TWeakObjectPtr<UAnimBank>& AnimBank : RegisteredAnimBanks)
		{
			if (AnimBank.IsValid())
			{
				AnimBanksToProcess.Add(AnimBank.Get());
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessFinishedAnimBanks);

			const double TickStartTime = FPlatformTime::Seconds();

			TSet<TWeakObjectPtr<UAnimBank>> AnimBanksToPostpone;
			TArray<UAnimBank*> ProcessedAnimBanks;
			
			if (AnimBanksToProcess.Num())
			{
				for (UAnimBank* AnimBank : AnimBanksToProcess)
				{
					const bool bHasBankUpdateLeft = ProcessedAnimBanks.Num() <= MaxBankUpdatesPerFrame;
					if (bHasBankUpdateLeft && AnimBank->IsAsyncTaskComplete())
					{
						PostCompilation(AnimBank);
						ProcessedAnimBanks.Add(AnimBank);
					}
					else
					{
						AnimBanksToPostpone.Emplace(AnimBank);
					}
				}
			}

			RegisteredAnimBanks = MoveTemp(AnimBanksToPostpone);
			PostCompilation(ProcessedAnimBanks);
		}
	}
}

void FAnimBankCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	FObjectCacheContextScope ObjectCacheScope;
	FinishCompilationsForGame();
	Reschedule();
	ProcessAnimBanks(bLimitExecutionTime);
	UpdateCompilationNotification();
}

#undef LOCTEXT_NAMESPACE

#endif // #if WITH_EDITOR
