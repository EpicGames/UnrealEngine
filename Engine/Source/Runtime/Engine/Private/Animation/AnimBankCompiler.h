// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "IAssetCompilingManager.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"

class FAsyncCompilationNotification;
class UAnimBank;
class FQueuedThreadPool;
struct FAssetCompileContext;
enum class EQueuedWorkPriority : uint8;

class FAnimBankCompilingManager : public IAssetCompilingManager
{
public:
	static FAnimBankCompilingManager& Get();

	/**
	* Returns the number of outstanding compilations.
	*/
	int32 GetNumRemainingAssets() const override;

	/**
	* Queue anim banks to be compiled asynchronously so they are monitored.
	*/
	void AddAnimBanks(TArrayView<UAnimBank* const> InAnimBanks);

	/**
	* Blocks until completion of the requested anim banks.
	*/
	void FinishCompilation(TArrayView<UAnimBank* const> InAnimBanks);

	/**
	* Blocks until completion of all async anim bank compilation.
	*/
	void FinishAllCompilation() override;

	/**
	* Returns the priority at which the given anim bank should be scheduled.
	*/
	EQueuedWorkPriority GetBasePriority(UAnimBank* InAnimBank) const;

	/**
	* Returns the thread pool where anim bank compilation should be scheduled.
	*/
	FQueuedThreadPool* GetThreadPool() const;

	/**
	* Cancel any pending work and blocks until it is safe to shut down.
	*/
	void Shutdown() override;

private:
	friend class FAssetCompilingManager;

	FAnimBankCompilingManager();

	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;

	/** Called once per frame, fetches completed tasks and applies them to the scene. */
	void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;

	bool bHasShutdown = false;
	TSet<TWeakObjectPtr<UAnimBank>> RegisteredAnimBanks;

	TUniquePtr<FAsyncCompilationNotification> Notification;

	void FinishCompilationsForGame();
	void Reschedule();
	void ProcessAnimBanks(bool bLimitExecutionTime, int32 MinBatchSize = 1);
	void UpdateCompilationNotification();

	void PostCompilation(TArrayView<UAnimBank* const> InAnimBanks);
	void PostCompilation(UAnimBank* InAnimBank);

	void OnPostReachabilityAnalysis();
	FDelegateHandle PostReachabilityAnalysisHandle;
};

#endif // WITH_EDITOR
