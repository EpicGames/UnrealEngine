// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimBlendDebugScope.h"
#include "Animation/AnimNodeBase.h"

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::Anim::FAnimBlendDebugScope);

namespace UE { namespace Anim {


FAnimBlendDebugScope::FAnimBlendDebugScope(const FAnimationBaseContext& InContext, const int32& InBlendIndex, const int32& InNumBlends, const FColor& InDebugColor)
	: BlendIndex(InBlendIndex)
	, NumBlends(InNumBlends)
	, DebugColor(InDebugColor)
	, NodeId(InContext.GetCurrentNodeId())
{
}


}}	// namespace UE::Anim
