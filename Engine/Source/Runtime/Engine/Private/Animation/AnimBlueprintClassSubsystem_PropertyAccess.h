// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimBlueprintClassSubsystem_PropertyAccess.generated.h"

// Dummy class to prevent warnings on load of older subsystem implementation
UCLASS(MinimalAPI)
class UAnimBlueprintClassSubsystem_PropertyAccess : public UObject
{
	GENERATED_BODY()
};