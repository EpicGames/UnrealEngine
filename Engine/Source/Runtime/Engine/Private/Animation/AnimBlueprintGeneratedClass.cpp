// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimBlueprintGeneratedClass.cpp: The object generated by compiling a AnimBlueprint
=============================================================================*/ 

#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Serialization/ObjectWriter.h"
#include "Animation/ExposedValueHandler.h"
#include "Serialization/ObjectReader.h"
#include "ObjectEditorUtils.h"
#include "UObject/AnimObjectVersion.h"
#include "UObject/Package.h"
#include "UObject/ReleaseObjectVersion.h"
#include "Animation/AnimNode_Root.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "Animation/AnimNode_LinkedAnimLayer.h"
#include "Animation/AnimNode_StateMachine.h"
#include "Animation/AnimSubsystemInstance.h"
#include "Engine/PoseWatch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintGeneratedClass)

/////////////////////////////////////////////////////
// FStateMachineDebugData

UEdGraphNode* FStateMachineDebugData::FindNodeFromStateIndex(int32 StateIndex) const
{
	if (const TWeakObjectPtr<UEdGraphNode>* pResult = NodeToStateIndex.FindKey(StateIndex))
	{
		return pResult->Get();
	}
	return NULL;
}

UEdGraphNode* FStateMachineDebugData::FindNodeFromTransitionIndex(int32 TransitionIndex) const
{
	if (const TWeakObjectPtr<UEdGraphNode>* pResult = NodeToTransitionIndex.FindKey(TransitionIndex))
	{
		return pResult->Get();
	}
	return NULL;
}

/////////////////////////////////////////////////////
// FAnimBlueprintDebugData

#if WITH_EDITORONLY_DATA

void FAnimBlueprintDebugData::TakeSnapshot(UAnimInstance* Instance)
{
	checkSlow(&(CastChecked<UAnimBlueprintGeneratedClass>((UObject*)(Instance->GetClass()))->GetAnimBlueprintDebugData()) == this);

	if (SnapshotBuffer == NULL)
	{
		SnapshotBuffer = new TSimpleRingBuffer<FAnimationFrameSnapshot>(30*30);
	}

	SnapshotBuffer->WriteNewElementInitialized().InitializeFromInstance(Instance);
}

void FAnimBlueprintDebugData::ResetSnapshotBuffer()
{
	if (SnapshotBuffer != NULL)
	{
		delete SnapshotBuffer;
		SnapshotBuffer = NULL;
	}
}

float FAnimBlueprintDebugData::GetSnapshotLengthInSeconds()
{
	if (SnapshotBuffer != NULL)
	{
		//@TODO: Shouldn't use hardcoded snapshot length; ideally we actually use timestamps, etc...
		return SnapshotBuffer->Num() * 1.0f / 30.0f;
	}

	return 0.0f;
}

int32 FAnimBlueprintDebugData::GetSnapshotLengthInFrames()
{
	if (SnapshotBuffer != NULL)
	{
		//@TODO: Shouldn't use hardcoded snapshot length; ideally we actually use timestamps, etc...
		return SnapshotBuffer->Num();
	}

	return 0;
}

void FAnimBlueprintDebugData::SetSnapshotIndex(UAnimInstance* Instance, int32 NewIndex)
{
	if (SnapshotBuffer != NULL)
	{
		int32 SavedIndex = SnapshotIndex;
		int32 ClampedIndex = FMath::Clamp<int32>(NewIndex, 0, SnapshotBuffer->Num() - 1);
		SnapshotIndex = (NewIndex == INDEX_NONE) ? INDEX_NONE : ClampedIndex;

		// Apply the desired snapshot (or the most recent one if free-running was selected)
		if ((SnapshotIndex != SavedIndex) && (SnapshotBuffer->Num() > 0))
		{
			(*SnapshotBuffer)(ClampedIndex).CopyToInstance(Instance);
		}
	}
}

void FAnimBlueprintDebugData::SetSnapshotIndexByTime(UAnimInstance* Instance, double TargetTime)
{
	const int32 SavedIndex = SnapshotIndex;

	int32 NewIndex = INDEX_NONE;

	if ((SnapshotBuffer != NULL) && (SnapshotBuffer->Num() > 0))
	{
		if ((TargetTime < (*SnapshotBuffer)(0).TimeStamp) && (TargetTime >= (*SnapshotBuffer)(SnapshotBuffer->Num()-1).TimeStamp))
		{
			for (NewIndex = SnapshotBuffer->Num() - 1; NewIndex > 0; --NewIndex)
			{
				if (TargetTime < (*SnapshotBuffer)(NewIndex-1).TimeStamp)
				{
					break;
				}
			}
		}
	}

	// Determine which snapshot to use
	SetSnapshotIndex(Instance, NewIndex);

	// Stomp on the time that was played back from the snapshot since we were given an exact one
	Instance->CurrentLifeTimerScrubPosition = TargetTime;
}

void FAnimBlueprintDebugData::ResetNodeVisitSites()
{
	UpdatedNodesThisFrame.Empty(UpdatedNodesThisFrame.Num());
	NodeInputAttributesThisFrame.Empty(NodeInputAttributesThisFrame.Num());
	NodeOutputAttributesThisFrame.Empty(NodeOutputAttributesThisFrame.Num());
	NodeSyncsThisFrame.Empty(NodeSyncsThisFrame.Num());
	StateData.Empty(StateData.Num());
	NodeValuesThisFrame.Empty(NodeValuesThisFrame.Num());
	SequencePlayerRecordsThisFrame.Empty(SequencePlayerRecordsThisFrame.Num());
	BlendSpacePlayerRecordsThisFrame.Empty(BlendSpacePlayerRecordsThisFrame.Num());
}

void FAnimBlueprintDebugData::RecordNodeVisit(int32 TargetNodeIndex, int32 SourceNodeIndex, float BlendWeight)
{
	UpdatedNodesThisFrame.Emplace(SourceNodeIndex, TargetNodeIndex, BlendWeight);
}

void FAnimBlueprintDebugData::RecordNodeVisitArray(const TArray<FNodeVisit>& Nodes)
{
	UpdatedNodesThisFrame.Append(Nodes);
}

void FAnimBlueprintDebugData::RecordNodeAttribute(int32 TargetNodeIndex, int32 SourceNodeIndex, FName InAttribute)
{
	if(TargetNodeIndex != INDEX_NONE)
	{
		TArray<FAnimBlueprintDebugData::FAttributeRecord>& InputAttributeRecords = NodeInputAttributesThisFrame.FindOrAdd(TargetNodeIndex);
		InputAttributeRecords.Emplace(SourceNodeIndex, InAttribute);
	}

	if(SourceNodeIndex != INDEX_NONE)
	{
		TArray<FAnimBlueprintDebugData::FAttributeRecord>& OutputAttributeRecords = NodeOutputAttributesThisFrame.FindOrAdd(SourceNodeIndex);
		OutputAttributeRecords.Emplace(TargetNodeIndex, InAttribute);
	}
}

void FAnimBlueprintDebugData::RecordNodeAttributeMaps(const TMap<int32, TArray<FAttributeRecord>>& InInputAttributes, const TMap<int32, TArray<FAttributeRecord>>& InOutputAttributes)
{
	if(NodeInputAttributesThisFrame.Num() == 0)
	{
		NodeInputAttributesThisFrame = InInputAttributes;
	}
	else
	{
		NodeInputAttributesThisFrame.Append(InInputAttributes);
	}

	if(NodeOutputAttributesThisFrame.Num() == 0)
	{
		NodeOutputAttributesThisFrame = InOutputAttributes;
	}
	else
	{
		NodeOutputAttributesThisFrame.Append(InOutputAttributes);
	}
}

void FAnimBlueprintDebugData::RecordNodeSync(int32 InSourceNodeIndex, FName InSyncGroup)
{
	NodeSyncsThisFrame.FindOrAdd(InSourceNodeIndex, InSyncGroup);
}

void FAnimBlueprintDebugData::RecordNodeSyncsArray(const TMap<int32, FName>& InNodeSyncs)
{	
	if(NodeSyncsThisFrame.Num() == 0)
	{
		NodeSyncsThisFrame = InNodeSyncs;
	}
	else
	{
		NodeSyncsThisFrame.Append(InNodeSyncs);
	}
}

void FAnimBlueprintDebugData::RecordStateData(int32 StateMachineIndex, int32 StateIndex, float Weight, float ElapsedTime)
{
	StateData.Emplace(StateMachineIndex, StateIndex, Weight, ElapsedTime);
}

void FAnimBlueprintDebugData::RecordNodeValue(int32 InNodeID, const FString& InText)
{
	NodeValuesThisFrame.Emplace(InText, InNodeID);
}

void FAnimBlueprintDebugData::RecordSequencePlayer(int32 InNodeID, float InPosition, float InLength, int32 InFrameCount)
{
	SequencePlayerRecordsThisFrame.Emplace(InNodeID, InPosition, InLength, InFrameCount);
}

void FAnimBlueprintDebugData::RecordBlendSpacePlayer(int32 InNodeID, const UBlendSpace* InBlendSpace, const FVector& InPosition, const FVector& InFilteredPosition)
{
	BlendSpacePlayerRecordsThisFrame.Emplace(InNodeID, InBlendSpace, InPosition, InFilteredPosition);
}

void FAnimBlueprintDebugData::AddPoseWatch(int32 NodeID, UPoseWatchPoseElement* const InPoseWatchPoseElement)
{
	for (FAnimNodePoseWatch& PoseWatch : AnimNodePoseWatch)
	{
		if (PoseWatch.NodeID == NodeID)
		{
			return;
		}
	}

	check(InPoseWatchPoseElement != nullptr); // Expect a valid pointer.

	//Not found so make new one
	AnimNodePoseWatch.Add(FAnimNodePoseWatch());
	FAnimNodePoseWatch& NewAnimNodePoseWatch = AnimNodePoseWatch.Last();
	NewAnimNodePoseWatch.NodeID = NodeID;
	NewAnimNodePoseWatch.PoseWatch = InPoseWatchPoseElement->GetParent();
	NewAnimNodePoseWatch.PoseWatchPoseElement = InPoseWatchPoseElement;
	NewAnimNodePoseWatch.Object = nullptr;
}

void FAnimBlueprintDebugData::RemovePoseWatch(int32 NodeID)
{
	for (int32 PoseWatchIdx = 0; PoseWatchIdx < AnimNodePoseWatch.Num(); ++PoseWatchIdx)
	{
		if (AnimNodePoseWatch[PoseWatchIdx].NodeID == NodeID)
		{
			AnimNodePoseWatch.RemoveAtSwap(PoseWatchIdx);
			return;
		}
	}
}

void FAnimBlueprintDebugData::ForEachActiveVisiblePoseWatchPoseElement(const TFunctionRef<void(FAnimNodePoseWatch&)>& InFunction)
{
	for (FAnimNodePoseWatch& PoseWatchNode : AnimNodePoseWatch)
	{
		if (const UPoseWatchPoseElement* PoseWatchPoseElement = PoseWatchNode.PoseWatchPoseElement)
		{
			if (PoseWatchPoseElement->GetIsEnabled() && PoseWatchPoseElement->GetIsVisible())
			{
				InFunction(PoseWatchNode);
			}
		}
	}
}

void FAnimBlueprintDebugData::DisableAllPoseWatches()
{
	for (FAnimNodePoseWatch& PoseWatchNode : AnimNodePoseWatch)
	{
		if (PoseWatchNode.PoseWatch)
		{
			PoseWatchNode.PoseWatch->SetIsNodeEnabled(false);
		}
	}
}

TArrayView<const FName> FAnimBlueprintDebugData::GetNodeAttributes(TWeakObjectPtr<UAnimGraphNode_Base> InAnimGraphNode) const
{
	const TArray<FName>* Attributes = NodeAttributes.Find(InAnimGraphNode);
	if(Attributes)
	{
		return MakeArrayView(*Attributes);
	}

	return TArrayView<const FName>();
}

/////////////////////////////////////////////////////
// FBinaryObjectWriter

class FBinaryObjectWriter : public FObjectWriter
{
public:
	FBinaryObjectWriter(UObject* Obj, TArray<uint8>& InBytes)
		: FObjectWriter(InBytes)
	{
		this->SetWantBinaryPropertySerialization(true);
		Obj->Serialize(*this);
	}
};

/////////////////////////////////////////////////////
// FBinaryObjectReader

class FBinaryObjectReader : public FObjectReader
{
public:
	FBinaryObjectReader(UObject* Obj, TArray<uint8>& InBytes)
		: FObjectReader(InBytes)
	{
		this->SetWantBinaryPropertySerialization(true);
		Obj->Serialize(*this);
	}
};

/////////////////////////////////////////////////////
// FAnimationFrameSnapshot

void FAnimationFrameSnapshot::InitializeFromInstance(UAnimInstance* Instance)
{
	FBinaryObjectWriter Writer(Instance, SerializedData);
	TimeStamp = Instance->LifeTimer;
}

void FAnimationFrameSnapshot::CopyToInstance(UAnimInstance* Instance)
{
	FBinaryObjectReader Reader(Instance, SerializedData);
	Instance->CurrentLifeTimerScrubPosition = TimeStamp;
}

#endif

/////////////////////////////////////////////////////
// UAnimBlueprintGeneratedClass

UAnimBlueprintGeneratedClass::UAnimBlueprintGeneratedClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const void* UAnimBlueprintGeneratedClass::GetConstantNodeValueRaw(int32 InIndex) const
{
	const FProperty* Property = ConstantProperties[InIndex];
	checkSlow(Property->GetOwner<UStruct>() && GetSparseClassDataStruct()->IsChildOf(Property->GetOwner<UStruct>()));
	const void* ConstantData = GetConstantNodeData();
	check(ConstantData);
	return Property->ContainerPtrToValuePtr<void>(static_cast<const void*>(ConstantData));
}

const void* UAnimBlueprintGeneratedClass::GetMutableNodeValueRaw(int32 InIndex, const UObject* InObject) const
{
	const FProperty* Property = MutableProperties[InIndex];
	checkSlow(Property->GetOwner<UStruct>() && Property->GetOwner<UStruct>()->IsChildOf(FAnimBlueprintMutableData::StaticStruct()));
	const FAnimBlueprintMutableData* MutableData = GetMutableNodeData(InObject);
	check(MutableData);	
	return Property->ContainerPtrToValuePtr<void>(static_cast<const void*>(MutableData));
}

FAnimBlueprintMutableData* UAnimBlueprintGeneratedClass::GetMutableNodeData(UObject* InObject) const
{
	if(MutableNodeDataProperty != nullptr)
	{
		checkSlow(MutableNodeDataProperty->Struct->IsChildOf(FAnimBlueprintMutableData::StaticStruct()));
		checkSlow(MutableNodeDataProperty->GetOwner<UClass>() && InObject->GetClass()->IsChildOf(MutableNodeDataProperty->GetOwner<UClass>()));
		return MutableNodeDataProperty->ContainerPtrToValuePtr<FAnimBlueprintMutableData>(InObject);
	}

	return nullptr;
}

const FAnimBlueprintMutableData* UAnimBlueprintGeneratedClass::GetMutableNodeData(const UObject* InObject) const
{
	if(MutableNodeDataProperty != nullptr)
	{
		checkSlow(MutableNodeDataProperty->Struct->IsChildOf(FAnimBlueprintMutableData::StaticStruct()));
		checkSlow(MutableNodeDataProperty->GetOwner<UClass>() && InObject->GetClass()->IsChildOf(MutableNodeDataProperty->GetOwner<UClass>()));
		return MutableNodeDataProperty->ContainerPtrToValuePtr<FAnimBlueprintMutableData>(InObject);
	}

	return nullptr;
}

const void* UAnimBlueprintGeneratedClass::GetConstantNodeData() const 
{
	return SparseClassData;
}

void UAnimBlueprintGeneratedClass::ForEachSubsystem(TFunctionRef<EAnimSubsystemEnumeration(const FAnimSubsystemContext&)> InFunction) const
{
	for(int32 SubsystemIndex = 0; SubsystemIndex < ConstantSubsystemProperties.Num(); ++SubsystemIndex)
	{
		FStructProperty* ConstantSubsystemProperty = ConstantSubsystemProperties[SubsystemIndex];
		checkSlow(ConstantSubsystemProperty->GetOwner<UStruct>() && GetSparseClassDataStruct()->IsChildOf(ConstantSubsystemProperty->GetOwner<UStruct>()));
		FAnimSubsystem& Subsystem = *ConstantSubsystemProperty->ContainerPtrToValuePtr<FAnimSubsystem>(const_cast<void*>(GetConstantNodeData()));

		if(InFunction(FAnimSubsystemContext(Subsystem, ConstantSubsystemProperty->Struct)) == EAnimSubsystemEnumeration::Stop)
		{
			break;
		}
	}
}

void UAnimBlueprintGeneratedClass::ForEachSubsystem(UObject* InObject, TFunctionRef<EAnimSubsystemEnumeration(const FAnimSubsystemInstanceContext&)> InFunction) const
{
	if(ConstantSubsystemProperties.Num() == MutableSubsystemProperties.Num())
	{
		for(int32 SubsystemIndex = 0; SubsystemIndex < ConstantSubsystemProperties.Num(); ++SubsystemIndex)
		{
			FStructProperty* ConstantSubsystemProperty = ConstantSubsystemProperties[SubsystemIndex];
			FStructProperty* MutableSubsystemProperty = MutableSubsystemProperties[SubsystemIndex];
			
			checkSlow(ConstantSubsystemProperty->GetOwner<UStruct>() && GetSparseClassDataStruct()->IsChildOf(ConstantSubsystemProperty->GetOwner<UStruct>()));
			const FAnimSubsystem& Subsystem = *ConstantSubsystemProperty->ContainerPtrToValuePtr<FAnimSubsystem>(GetConstantNodeData());

			checkSlow(MutableSubsystemProperty->GetOwner<UClass>() && InObject->GetClass()->IsChildOf(MutableSubsystemProperty->GetOwner<UClass>()));
			FAnimSubsystemInstance& SubsystemInstance = *MutableSubsystemProperty->ContainerPtrToValuePtr<FAnimSubsystemInstance>(InObject);
			
			if(InFunction(FAnimSubsystemInstanceContext(Subsystem, ConstantSubsystemProperty->Struct, SubsystemInstance, MutableSubsystemProperty->Struct)) == EAnimSubsystemEnumeration::Stop)
			{
				break;
			}
		}
	}
}

const FAnimSubsystem* UAnimBlueprintGeneratedClass::FindSubsystem(UScriptStruct* InSubsystemType) const
{
	const FAnimSubsystem* FoundSubsystem = nullptr;
	
	ForEachSubsystem([&FoundSubsystem, InSubsystemType](const FAnimSubsystemContext& InContext)
	{
		if(InContext.SubsystemStruct == InSubsystemType)
		{
			FoundSubsystem = &InContext.Subsystem;
			return EAnimSubsystemEnumeration::Stop;
		}

		return EAnimSubsystemEnumeration::Continue;
	});

	return FoundSubsystem;
}

void UAnimBlueprintGeneratedClass::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	Super::Serialize(Ar);
}

void UAnimBlueprintGeneratedClass::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
#if WITH_EDITOR
	// Re-link the mutables struct if we have one as we need to correctly account for its size
	// This is mostly needed during re-instancing as post duplication the class requires a re-link
	if(UScriptStruct* MutablesStruct = FindObject<UScriptStruct>(this, *GetMutablesStructName().ToString()))
	{
		MutablesStruct->Link(Ar, bRelinkExistingProperties);
	}
#endif	
	
	Super::Link(Ar, bRelinkExistingProperties);
	
	// @TODO: Shouldn't be necessary to clear these, but currently the class gets linked twice during compilation
	AnimNodeProperties.Empty();
	LinkedAnimGraphNodeProperties.Empty();
	LinkedAnimLayerNodeProperties.Empty();
	PreUpdateNodeProperties.Empty();
	DynamicResetNodeProperties.Empty();
	StateMachineNodeProperties.Empty();
	InitializationNodeProperties.Empty();
	MutableProperties.Empty();
	ConstantProperties.Empty();
	MutableSubsystemProperties.Empty();
	ConstantSubsystemProperties.Empty();
	MutableNodeDataProperty = nullptr;

#if WITH_EDITOR
	// This relies on the entire class being fully loaded, this is not the case with EDL async-loading, in which case the functions are generated in PostLoad
	GenerateAnimationBlueprintFunctions();
#endif // WITH_EDITOR

	// Initialize the various tracked node arrays & fix up function internals
	for (TFieldIterator<FProperty> It(this); It; ++It)
	{
		if (FStructProperty* StructProp = CastField<FStructProperty>(*It))
		{
			if (StructProp->Struct->IsChildOf(FAnimNode_Base::StaticStruct()))
			{
				if(StructProp->Struct == FAnimNode_LinkedAnimGraph::StaticStruct())
				{
					LinkedAnimGraphNodeProperties.Add(StructProp);
				}
				else if(StructProp->Struct == FAnimNode_LinkedAnimLayer::StaticStruct())
				{
					LinkedAnimLayerNodeProperties.Add(StructProp);
				}
				else if(StructProp->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
				{
					StateMachineNodeProperties.Add(StructProp);
				}
				AnimNodeProperties.Add(StructProp);
			}
			else if(StructProp->Struct->IsChildOf(FAnimBlueprintMutableData::StaticStruct()))
			{
				MutableNodeDataProperty = StructProp;

				for(TFieldIterator<FProperty> MutableIt(StructProp->Struct); MutableIt; ++MutableIt)
				{
					MutableProperties.Add(*MutableIt);
				}
			}
			else if(StructProp->Struct->IsChildOf(FAnimSubsystemInstance::StaticStruct()))
			{
				MutableSubsystemProperties.Add(StructProp);
			}
		}
	}

#if WITH_EDITORONLY_DATA
	// With cooked data, we skip this as Link() is called on serialization and sparse class data
	// is not properly serialized yet
	if(!GetPackage()->bIsCookedForEditor)
	{
		// Must build constant properties to be able to iterate subsystems and call OnLink()
		// Subsystems in non-editor data builds should be initialized in OnPostLoadDefaults.
		BuildConstantProperties();
		
		ForEachSubsystem([this](const FAnimSubsystemContext& InContext)
		{
			Subsystems.Add(&InContext.Subsystem);

			FAnimSubsystemLinkContext Context(InContext, *this);
			const_cast<FAnimSubsystem&>(InContext.Subsystem).OnLink(Context);
			return EAnimSubsystemEnumeration::Continue;
		});
	}
#endif
}

void UAnimBlueprintGeneratedClass::PurgeClass(bool bRecompilingOnLoad)
{
	Super::PurgeClass(bRecompilingOnLoad);

	AnimNotifies.Empty();
	TargetSkeleton = NULL;
#if WITH_EDITORONLY_DATA
	AnimBlueprintDebugData = FAnimBlueprintDebugData();
#endif

	BakedStateMachines.Empty();
}

uint8* UAnimBlueprintGeneratedClass::GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const
{
	if(!IsInGameThread())
	{
		// we cant use the persistent frame if we are executing in parallel (as we could potentially thunk to BP)
		return nullptr;
	}

	return Super::GetPersistentUberGraphFrame(Obj, FuncToCheck);
}

#if WITH_EDITOR
void UAnimBlueprintGeneratedClass::HandleReinitializeObjectAfterCompile(UObject* Object) const
{
	InitializeAnimNodeData(Object, true);
}

void UAnimBlueprintGeneratedClass::PrepareToConformSparseClassData(UScriptStruct* SparseClassDataArchetypeStruct)
{
	// ABPGC sparse data is generated on compile, so there is nothing to conform
}

void UAnimBlueprintGeneratedClass::ConformSparseClassData(UObject* Object)
{
	// ABPGC sparse data is generated on compile, so there is nothing to conform
}
#endif

void UAnimBlueprintGeneratedClass::PostLoadDefaultObject(UObject* Object)
{
	Super::PostLoadDefaultObject(Object);

	OnPostLoadDefaults(Object);
}

void UAnimBlueprintGeneratedClass::OnPostLoadDefaults(UObject* Object)
{
	// Make sure that 'functions' are generated before we use them in LinkFunctionsToDefaultObjectNodes()
	if (AnimBlueprintFunctions.IsEmpty()) 
	{ 
		GenerateAnimationBlueprintFunctions();
	}

	BuildConstantProperties();

	LinkFunctionsToDefaultObjectNodes(Object);

	Subsystems.Empty();
	
	ForEachSubsystem(Object, [this, Object](const FAnimSubsystemInstanceContext& InContext)
	{
		Subsystems.Add(&InContext.Subsystem);

		FAnimSubsystemPostLoadDefaultsContext Context(InContext, Object);
		const_cast<FAnimSubsystem&>(InContext.Subsystem).OnPostLoadDefaults(Context);
		return EAnimSubsystemEnumeration::Continue;
	});
}

void UAnimBlueprintGeneratedClass::PostLoad()
{
	Super::PostLoad();

	BuildConstantProperties();

	// Call postload on the class subsystems
	ForEachSubsystem([this](const FAnimSubsystemContext& InContext)
	{
		FAnimSubsystemPostLoadContext Context(InContext, *this);
		const_cast<FAnimSubsystem&>(InContext.Subsystem).OnPostLoad(Context);
		return EAnimSubsystemEnumeration::Continue;
	});
}

void UAnimBlueprintGeneratedClass::GenerateAnimationBlueprintFunctions()
{
	AnimBlueprintFunctions.Empty();

	static const FName DefaultAnimGraphName("AnimGraph");
	// Patch up blueprint function info
	for (TFieldIterator<UFunction> It(this); It; ++It)
	{
		bool bFoundOutput = false;
#if WITH_EDITOR
		// In editor we can grab the group from metadata, otherwise we need to wait until CDO post load (LinkFunctionsToDefaultObjectNodes)
		FText CategoryText = FObjectEditorUtils::GetCategoryText(*It);
		FName Group = CategoryText.IsEmpty() ? NAME_None : FName(*CategoryText.ToString());
#endif
		FStructProperty* OutputPoseNodeProperty = nullptr;
		TArray<FName> InputPoseNames;
		TArray<int32> InputPoseNodeIndices;
		TArray<FStructProperty*> InputPoseNodeProperties;
		TArray<FAnimBlueprintFunction::FInputPropertyData> InputPropertyData;

		// grab the input/output poses, their indices will be patched up later once the CDO is loaded in PostLoadDefaultObject
		for (TFieldIterator<FProperty> ItParam(*It); ItParam; ++ItParam)
		{
			if (FStructProperty* StructProperty = CastField<FStructProperty>(*ItParam))
			{
				if (StructProperty->Struct->IsChildOf(FPoseLink::StaticStruct()))
				{
					if (StructProperty->GetPropertyFlags() & CPF_OutParm)
					{
						if (!bFoundOutput)
						{
							OutputPoseNodeProperty = StructProperty;
							bFoundOutput = true;
						}
						else
						{
							// our required signature needs us to have a single post link output, so null it out if we find more than one
							OutputPoseNodeProperty = nullptr;
						}
					}
					else
					{
						InputPoseNames.Add(StructProperty->GetFName());
						InputPoseNodeIndices.Add(INDEX_NONE);
						InputPoseNodeProperties.Add(nullptr);
					}
				}
				else
				{
					InputPropertyData.Add({ ItParam->GetFName(), *ItParam, nullptr });
				}
			}
			else
			{
				InputPropertyData.Add({ ItParam->GetFName(), *ItParam, nullptr });
			}
		}

		if (OutputPoseNodeProperty)
		{
			// We use the undecorated name here, so trim the postfix
			FAnimBlueprintFunction* AnimBlueprintFunction = nullptr;

			FName FunctionName = It->GetFName();

			// Make sure that the default graph is at index 0
			if (FunctionName == DefaultAnimGraphName)
			{
				AnimBlueprintFunction = &AnimBlueprintFunctions.Insert_GetRef(FAnimBlueprintFunction(FunctionName), 0);
			}
			else
			{
				AnimBlueprintFunction = &AnimBlueprintFunctions.Emplace_GetRef(FunctionName);
			}

#if WITH_EDITOR
			AnimBlueprintFunction->Group = Group;
#endif
			AnimBlueprintFunction->InputPoseNames.Append(MoveTemp(InputPoseNames));
			AnimBlueprintFunction->InputPoseNodeIndices.Append(MoveTemp(InputPoseNodeIndices));
			AnimBlueprintFunction->InputPoseNodeProperties.Append(MoveTemp(InputPoseNodeProperties));
			AnimBlueprintFunction->InputPropertyData.Append(MoveTemp(InputPropertyData));
		}
	}
}

void UAnimBlueprintGeneratedClass::InitializeAnimNodeData(UObject* DefaultObject, bool bForce) const
{
#if WITH_EDITOR
	// In editor, skip this work if properties and node data are mismatched. This will be rectified by compile-on-load
	if (AnimNodeProperties.Num() != AnimNodeData.Num())
	{
		return;
	}
#endif

	// Link functions to their nodes
	for(int32 AnimNodeIndex = 0; AnimNodeIndex < AnimNodeProperties.Num() && AnimNodeIndex < AnimNodeData.Num(); ++AnimNodeIndex)
	{
		FStructProperty* StructProperty = AnimNodeProperties[AnimNodeIndex];

		if(StructProperty->Struct->IsChildOf(FAnimNode_Base::StaticStruct()))
		{
			FAnimNode_Base* Node = StructProperty->ContainerPtrToValuePtr<FAnimNode_Base>(DefaultObject);

			// This function is called on child->parent hierarchies in postload. We dont want to overwrite child data
			// with parent data so skip if the node data has already been set up by a previous call.	
			if(bForce || Node->NodeData == nullptr || IAnimClassInterface::GetActualAnimClass(&Node->NodeData->GetAnimClassInterface()) != DefaultObject->GetClass())
			{
				Node->SetNodeData(AnimNodeData[AnimNodeIndex]);
			}
		}
	}
}

void UAnimBlueprintGeneratedClass::LinkFunctionsToDefaultObjectNodes(UObject* DefaultObject)
{
	PreUpdateNodeProperties.Empty();
	DynamicResetNodeProperties.Empty();
	InitializationNodeProperties.Empty();

#if WITH_EDITORONLY_DATA
	// If this class has not been generated against the current set of anim node data layouts, accessing anim node data
	// (e.g. via RootNode->GetName()) could end up accessing incorrect data. To verify this we check the serialized layout
	// that we are expecting to find here and do not use it if so.
	// This is OK in editor due to compile-on-load rectifying the layout against the current set of anim node structs.
	// If we continue past this point then we could potentially pollute any persistent FNodeDataIds with invalid indices.
	if(!VerifyNodeDataLayout())
	{
		return;
	}
#endif

	// Perform node init as a first pass as root nodes (needed for function names)
	// may not be initialized in sparse class data before they need to be patched
	InitializeAnimNodeData(DefaultObject, false);
	
	// Link functions to their nodes
	for(int32 AnimNodeIndex = 0; AnimNodeIndex < AnimNodeProperties.Num(); ++AnimNodeIndex)
	{
		FStructProperty* StructProperty = AnimNodeProperties[AnimNodeIndex];

		if(StructProperty->Struct->IsChildOf(FAnimNode_Base::StaticStruct()))
		{
			FAnimNode_Base* Node = StructProperty->ContainerPtrToValuePtr<FAnimNode_Base>(DefaultObject);

			if(Node->NeedsDynamicReset())
			{
				DynamicResetNodeProperties.Add(StructProperty);
			}

			if(Node->HasPreUpdate())
			{
				PreUpdateNodeProperties.Add(StructProperty);
			}

			if(Node->NeedsOnInitializeAnimInstance())
			{
				InitializationNodeProperties.Add(StructProperty);
			}
		}
		
		if (StructProperty->Struct->IsChildOf(FAnimNode_Root::StaticStruct()))
		{
			FAnimNode_Root* RootNode = StructProperty->ContainerPtrToValuePtr<FAnimNode_Root>(DefaultObject);
			const FName RootNodeName = RootNode->GetName();
			if(FAnimBlueprintFunction* FoundFunction = AnimBlueprintFunctions.FindByPredicate([RootNodeName](const FAnimBlueprintFunction& InFunction){ return InFunction.Name == RootNodeName; }))
			{
				FoundFunction->Group = RootNode->GetGroup();
				FoundFunction->OutputPoseNodeIndex = AnimNodeIndex;
				FoundFunction->OutputPoseNodeProperty = StructProperty;
			}
		}
		else if(StructProperty->Struct->IsChildOf(FAnimNode_LinkedInputPose::StaticStruct()))
		{
			FAnimNode_LinkedInputPose* LinkedInputPoseNode = StructProperty->ContainerPtrToValuePtr<FAnimNode_LinkedInputPose>(DefaultObject);
			if(FAnimBlueprintFunction* FoundFunction = AnimBlueprintFunctions.FindByPredicate([LinkedInputPoseNode](const FAnimBlueprintFunction& InFunction){ return InFunction.Name == LinkedInputPoseNode->Graph; }))
			{
				for(int32 InputIndex = 0; InputIndex < FoundFunction->InputPoseNames.Num(); ++InputIndex)
				{
					if(FoundFunction->InputPoseNames[InputIndex] == LinkedInputPoseNode->Name)
					{
						FoundFunction->InputPoseNodeIndices[InputIndex] = AnimNodeIndex;
						FoundFunction->InputPoseNodeProperties[InputIndex] = StructProperty;
					}
				}

				for(int32 InputPropertyIndex = 0; InputPropertyIndex < FoundFunction->InputPropertyData.Num(); ++InputPropertyIndex)
				{
					FoundFunction->InputPropertyData[InputPropertyIndex].ClassProperty = FindPropertyByName(FoundFunction->InputPropertyData[InputPropertyIndex].Name);
				}
			}
		}
	}

	// Determine whether functions are 'implemented'
	for(FAnimBlueprintFunction& AnimBlueprintFunction : AnimBlueprintFunctions)
	{
		if(AnimBlueprintFunction.OutputPoseNodeProperty)
		{
			FAnimNode_Root* RootNode = AnimBlueprintFunction.OutputPoseNodeProperty->ContainerPtrToValuePtr<FAnimNode_Root>(DefaultObject);
			if(RootNode->Result.LinkID != INDEX_NONE)
			{
				AnimBlueprintFunction.bImplemented = true;
			}
		}
	}

#if DO_CHECK
	if(!(GetClassFlags() | CLASS_Transient))
	{
		// Now verify we fixed up all our functions
		for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimBlueprintFunctions)
		{
			check(AnimBlueprintFunction.Name != NAME_None);
			check(AnimBlueprintFunction.OutputPoseNodeIndex != INDEX_NONE);
			check(AnimBlueprintFunction.OutputPoseNodeProperty != nullptr);
			check(AnimBlueprintFunction.InputPoseNames.Num() == AnimBlueprintFunction.InputPoseNodeIndices.Num());
			check(AnimBlueprintFunction.InputPoseNames.Num() == AnimBlueprintFunction.InputPoseNodeProperties.Num());

			for(int32 InputIndex = 0; InputIndex < AnimBlueprintFunction.InputPoseNames.Num(); ++InputIndex)
			{
				check(AnimBlueprintFunction.InputPoseNames[InputIndex] != NAME_None);
				check(AnimBlueprintFunction.InputPoseNodeIndices[InputIndex] != INDEX_NONE);
				check(AnimBlueprintFunction.InputPoseNodeProperties[InputIndex] != nullptr);
			}
		}
	}
#endif
}

#if WITH_EDITORONLY_DATA

const int32* UAnimBlueprintGeneratedClass::GetNodePropertyIndexFromGuid(FGuid Guid, EPropertySearchMode::Type SearchMode /*= EPropertySearchMode::OnlyThis*/)
{
	if (SearchMode == EPropertySearchMode::OnlyThis)
	{
		return AnimBlueprintDebugData.NodeGuidToIndexMap.Find(Guid);
	}
	else
	{
		TArray<const UBlueprintGeneratedClass*> BlueprintHierarchy;
		GetGeneratedClassesHierarchy(this, BlueprintHierarchy);

		for (const UBlueprintGeneratedClass* Blueprint : BlueprintHierarchy)
		{
			if (const UAnimBlueprintGeneratedClass* AnimBlueprintClass = Cast<UAnimBlueprintGeneratedClass>(Blueprint))
			{
				const int32* NodeIndex = AnimBlueprintClass->AnimBlueprintDebugData.NodeGuidToIndexMap.Find(Guid);

				if (NodeIndex)
				{
					return NodeIndex;
				}
			}
		}
	}

	return NULL;
}

int32 UAnimBlueprintGeneratedClass::GetNodeIndexFromGuid(FGuid Guid, EPropertySearchMode::Type SearchMode)
{
	if(const int32* IndexPtr = GetNodePropertyIndexFromGuid(Guid, SearchMode))
	{
		return AnimNodeProperties.Num() - 1 - *IndexPtr;
	}

	return INDEX_NONE;
}

const UEdGraphNode* UAnimBlueprintGeneratedClass::GetVisualNodeFromNodePropertyIndex(int32 PropertyIndex, EPropertySearchMode::Type SearchMode) const
{
	const int32 ReversedPropertyIndex = AnimNodeProperties.Num() - PropertyIndex - 1;
	if (SearchMode == EPropertySearchMode::OnlyThis)
	{
		const TWeakObjectPtr<const UEdGraphNode>* Node = AnimBlueprintDebugData.NodePropertyIndexToNodeMap.Find(ReversedPropertyIndex);
		return Node ? Node->Get() : nullptr;
	}
	else
	{
		TArray<const UBlueprintGeneratedClass*> BlueprintHierarchy;
		GetGeneratedClassesHierarchy(this, BlueprintHierarchy);

		for (const UBlueprintGeneratedClass* Blueprint : BlueprintHierarchy)
		{
			if (const UAnimBlueprintGeneratedClass* AnimBlueprintClass = Cast<UAnimBlueprintGeneratedClass>(Blueprint))
			{
				const TWeakObjectPtr<const UEdGraphNode>* Node = AnimBlueprintClass->AnimBlueprintDebugData.NodePropertyIndexToNodeMap.Find(ReversedPropertyIndex);
				if (Node)
				{
					return Node->Get();
				}
			}
		}
	}

	return nullptr;
}

#endif // WITH_EDITORONLY_DATA

void UAnimBlueprintGeneratedClass::BuildConstantProperties()
{
	ConstantProperties.Empty();
	ConstantSubsystemProperties.Empty();

	if(UScriptStruct* ConstantsStruct = GetSparseClassDataStruct())
	{
		// Make sure we create the data area up-front here
		GetOrCreateSparseClassData();

		// We must init sparse class data here once the class has been serialized
		for (TFieldIterator<FProperty> It(ConstantsStruct); It; ++It)
		{
			if (FStructProperty* StructProp = CastField<FStructProperty>(*It))
			{
				if(StructProp->Struct->IsChildOf(FAnimSubsystem::StaticStruct()))
				{
					ConstantSubsystemProperties.Add(StructProp);
				}
				// Initialize any anim node functions held in sparse data
				else if(StructProp->Struct == FAnimNodeFunctionRef::StaticStruct())
				{
					FAnimNodeFunctionRef& AnimNodeFunction = *StructProp->ContainerPtrToValuePtr<FAnimNodeFunctionRef>(SparseClassData);
					AnimNodeFunction.Initialize(this);
				}
			}
			
			ConstantProperties.Add(*It);
		}
	}
}

int32 UAnimBlueprintGeneratedClass::GetAnimNodePropertyIndex(const UScriptStruct* InNodeType, FName InPropertyName) const
{
	const FAnimNodeStructData* ScriptStructDataPtr = NodeTypeMap.Find(InNodeType);
	if(ScriptStructDataPtr != nullptr)
	{
		return ScriptStructDataPtr->GetPropertyIndex(InPropertyName);
	}

	return INDEX_NONE;
}

int32 UAnimBlueprintGeneratedClass::GetAnimNodePropertyCount(const UScriptStruct* InNodeType) const
{
	const FAnimNodeStructData* ScriptStructDataPtr = NodeTypeMap.Find(InNodeType);
	if(ScriptStructDataPtr != nullptr)
	{
		return ScriptStructDataPtr->GetNumProperties();
	}
	
	return 0;
}

FName UAnimBlueprintGeneratedClass::GetConstantsStructName()
{
	static const FName Name("AnimBlueprintGeneratedConstantData");
	return Name;
}

FName UAnimBlueprintGeneratedClass::GetMutablesStructName()
{
	static const FName Name("AnimBlueprintGeneratedMutableData");
	return Name;
}

#if WITH_EDITORONLY_DATA
bool UAnimBlueprintGeneratedClass::VerifyNodeDataLayout()
{
	// Run though the type map and check against the currently-compiled set of structs
	for(const auto& NodeTypePair : NodeTypeMap)
	{
		FAnimNodeStructData CurrentData(NodeTypePair.Key);
		const FAnimNodeStructData& CompiledInData = NodeTypePair.Value;

		if(!CurrentData.DoesLayoutMatch(CompiledInData))
		{
			bDataLayoutValid = false;
			return false;
		}
	}

	bDataLayoutValid = true;
	return true;
}
#endif
