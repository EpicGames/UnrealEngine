// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimBoneCompressionCodec.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimBoneDecompressionData.h"
#include "Animation/AnimSequence.h"
#include "Misc/MemStack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBoneCompressionCodec)

UAnimBoneCompressionCodec::UAnimBoneCompressionCodec(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Description(TEXT("None"))
{
}

#if WITH_EDITORONLY_DATA
int64 UAnimBoneCompressionCodec::EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const
{
#if WITH_EDITOR
	// This is a conservative estimate that gives a codec enough space to create two raw copies of the input data.
	return 2 * int64(AnimSequence.GetApproxBoneRawSize());
#else
	return -1;
#endif // WITH_EDITOR
}
#endif // WITH_EDITORONLY_DATA

UAnimBoneCompressionCodec* UAnimBoneCompressionCodec::GetCodec(const FString& DDCHandle)
{
	const FString ThisHandle = GetCodecDDCHandle();
	return ThisHandle == DDCHandle ? this : nullptr;
}

FString UAnimBoneCompressionCodec::GetCodecDDCHandle() const
{
	// In the DDC, we store a handle to this codec. It must be unique within the parent settings asset
	// and all children/sibling codecs. Imagine we have a settings asset with codec A and B.
	// A sequence is compressed with it and selects codec B.
	// The settings asset is duplicated. It will have the same DDC key and the data will not re-compress.
	// When we attempt to load from the DDC, we will have a handle created with the original settings
	// asset pointing to codec B. We must be able to find this codec B in the duplicated asset as well.

	FString Handle;
	Handle.Reserve(128);

	GetFName().AppendString(Handle);

	const UObject* Obj = GetOuter();
	while (Obj != nullptr && !Obj->IsA<UAnimBoneCompressionSettings>())
	{
		Handle += TEXT(".");

		Obj->GetFName().AppendString(Handle);

		Obj = Obj->GetOuter();
	}

	return Handle;
}

#if WITH_EDITORONLY_DATA
void UAnimBoneCompressionCodec::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
}
#endif


// Default implementation that codecs should override and implement in a more performant way
// It performs a conversion between SoA and AoS, making copies of the data between the formats
void UAnimBoneCompressionCodec::DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const UE::Anim::FAnimPoseDecompressionData& DecompressionData) const
{
	const int32 NumTransforms = DecompressionData.GetOutAtomRotations().Num();

	FMemMark Mark(FMemStack::Get());

	TArray<FTransform, TMemStackAllocator<>> OutAtoms;
	OutAtoms.SetNum(NumTransforms);

	// Right now we only support same size arrays
	check(DecompressionData.GetOutAtomRotations().Num() == DecompressionData.GetOutAtomTranslations().Num() && DecompressionData.GetOutAtomRotations().Num() == DecompressionData.GetOutAtomScales3D().Num());

	// Copy atoms data into the FTransform array
	ITERATE_NON_OVERLAPPING_ARRAYS_START(FQuat, DecompressionData.GetOutAtomRotations(), FTransform, OutAtoms, NumTransforms)
		ItSecond->SetRotation(*ItFirst);
	ITERATE_NON_OVERLAPPING_ARRAYS_END()

	ITERATE_NON_OVERLAPPING_ARRAYS_START(FVector, DecompressionData.GetOutAtomTranslations(), FTransform, OutAtoms, NumTransforms)
		ItSecond->SetTranslation(*ItFirst);
	ITERATE_NON_OVERLAPPING_ARRAYS_END()

	ITERATE_NON_OVERLAPPING_ARRAYS_START(FVector, DecompressionData.GetOutAtomScales3D(), FTransform, OutAtoms, NumTransforms)
		ItSecond->SetScale3D(*ItFirst);
	ITERATE_NON_OVERLAPPING_ARRAYS_END()

	// Decompress using default FTransform function
	TArrayView<FTransform> OutAtomsView = OutAtoms;
	DecompressPose(DecompContext, DecompressionData.GetRotationPairs(), DecompressionData.GetTranslationPairs(), DecompressionData.GetScalePairs(), OutAtomsView);

	// Copy back the result
	ITERATE_NON_OVERLAPPING_ARRAYS_START(FTransform, OutAtoms, FQuat, DecompressionData.GetOutAtomRotations(), NumTransforms)
		*ItSecond = ItFirst->GetRotation();
	ITERATE_NON_OVERLAPPING_ARRAYS_END()

	ITERATE_NON_OVERLAPPING_ARRAYS_START(FTransform, OutAtoms, FVector, DecompressionData.GetOutAtomTranslations(), NumTransforms)
		*ItSecond = ItFirst->GetTranslation();
	ITERATE_NON_OVERLAPPING_ARRAYS_END()

	ITERATE_NON_OVERLAPPING_ARRAYS_START(FTransform, OutAtoms, FVector, DecompressionData.GetOutAtomScales3D(), NumTransforms)
		*ItSecond = ItFirst->GetScale3D();
	ITERATE_NON_OVERLAPPING_ARRAYS_END()
}
