// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimBoneCompressionCodec.h"
#include "Animation/AnimationSettings.h"
#include "AnimationUtils.h"
#include "AnimationCompression.h"
#include "Async/ParallelFor.h"
#include "Async/UniqueLock.h"
#include "Async/SharedLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBoneCompressionSettings)

#define DEBUG_DUMP_ANIM_COMPRESSION_STATS 0


UAnimBoneCompressionSettings::UAnimBoneCompressionSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	ErrorThreshold = 0.1f;

	UAnimationSettings* AnimSetting = UAnimationSettings::Get();
	bForceBelowThreshold = AnimSetting->bForceBelowThreshold;
#endif
}

UAnimBoneCompressionCodec* UAnimBoneCompressionSettings::GetCodec(const FString& DDCHandle)
{
	UE::TSharedLock<UE::FSharedMutex> ReadLock(CodecsLock);

	UAnimBoneCompressionCodec* CodecMatch = nullptr;
	for (UAnimBoneCompressionCodec* Codec : Codecs)
	{
		if (Codec != nullptr)
		{
			CodecMatch = Codec->GetCodec(DDCHandle);

			if (CodecMatch != nullptr)
			{
				break;	// Found our match
			}
		}
	}

	return CodecMatch;
}

void UAnimBoneCompressionSettings::PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph)
{
	// The Codecs array is read by GetCodec() during UAnimSequence::Serialize on async loading threads.
	// FArrayProperty::InstanceSubobjects memmoves array elements causing a race
	UE::TUniqueLock<UE::FSharedMutex> WriteLock(CodecsLock);
	Super::PostLoadSubobjects(OuterInstanceGraph);
}

#if WITH_EDITORONLY_DATA
void UAnimBoneCompressionSettings::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	for (UAnimBoneCompressionCodec* Codec : Codecs)
	{
		if (Codec != nullptr)
		{
			OutDeps.Add(Codec);
		}
	}
}

bool UAnimBoneCompressionSettings::AreSettingsValid() const
{
	int32 NumValidCodecs = 0;

	for (const UAnimBoneCompressionCodec* Codec : Codecs)
	{
		if (Codec != nullptr)
		{
			// If one codec isn't valid, the settings are invalid
			if (!Codec->IsCodecValid())
			{
				return false;
			}

			NumValidCodecs++;
		}
	}

	return NumValidCodecs != 0;
}

bool UAnimBoneCompressionSettings::IsHighFidelity(const FCompressibleAnimData& CompressibleAnimData) const
{
	for (const UAnimBoneCompressionCodec* Codec : Codecs)
	{
		if (Codec != nullptr && Codec->IsHighFidelity(CompressibleAnimData))
		{
			// At least one codec is high fidelity
			return true;
		}
	}

	return false;
}

struct FAnimBoneCompressionContext
{
	/** The animation sequence we are compressing. */
	const FCompressibleAnimData& AnimSeq;

	/** The codec used to compress with. Note that the codec contained within the result struct can end up different. */
	UAnimBoneCompressionCodec* Codec;

	/** The compression result. */
	FCompressibleAnimDataResult Result;
	bool bSuccess;

	FAnimBoneCompressionContext(const FCompressibleAnimData& AnimSeq_, UAnimBoneCompressionCodec* Codec_)
		: AnimSeq(AnimSeq_)
		, Codec(Codec_)
		, Result()
		, bSuccess(false)
	{}

	int32 GetCompressedSize() const { return Result.AnimData != nullptr ? Result.AnimData->GetApproxCompressedSize() : 0; }
};

static void CompressAnimSequenceImpl(FAnimBoneCompressionContext& Context)
{
	Context.bSuccess = Context.Codec->Compress(Context.AnimSeq, Context.Result);

	if (Context.bSuccess && !Context.AnimSeq.IsCancelled())
	{
		FAnimationUtils::ComputeCompressionError(Context.AnimSeq, Context.Result, Context.Result.AnimData->BoneCompressionErrorStats);
	}
}

bool UAnimBoneCompressionSettings::Compress(const FCompressibleAnimData& AnimSeq, FCompressibleAnimDataResult& OutCompressedData) const
{
	if (!AreSettingsValid())
	{
		return false;
	}

	if (AnimSeq.RawAnimationData.Num() == 0)
	{
		// this can happen if the animation only contains curve - i.e. blendshape curves
		// when this happens, compression is considered a success but no codec is selected as none is needed
		return true;
	}

#if DEBUG_DUMP_ANIM_COMPRESSION_STATS
	const double CompressionStartTime = FPlatformTime::Seconds();
#endif

	TArray<FAnimBoneCompressionContext*> ContextList;

	for (UAnimBoneCompressionCodec* Codec : Codecs)
	{
		if (Codec == nullptr)
		{
			continue;
		}

		FAnimBoneCompressionContext* Context = new FAnimBoneCompressionContext(AnimSeq, Codec);
		ContextList.Add(Context);
	}

	ParallelForTemplate(ContextList.Num(), [&ContextList](int32 TaskIndex)
	{
		CompressAnimSequenceImpl(*ContextList[TaskIndex]);
	}, EParallelForFlags::Unbalanced);

	const int32 NumContextes = ContextList.Num();
	if (NumContextes == 0)
	{
		return false;
	}

	FAnimBoneCompressionContext* BestContext = nullptr;
	int32 ComparisonLoopStart = 0;

	for (int32 ContextIndex = 0; ContextIndex < NumContextes; ++ContextIndex)
	{
		FAnimBoneCompressionContext* Context = ContextList[ContextIndex];

		if (Context->bSuccess)
		{
			BestContext = Context;
			ComparisonLoopStart = ContextIndex + 1;
			break;
		}
	}

	if(BestContext)
	{
		const int32 RawSize = AnimSeq.GetApproxRawSize();

		// Find the best codec
		int32 BestSize = BestContext->GetCompressedSize();

		AnimationErrorStats BestErrorStats = BestContext->Result.AnimData->BoneCompressionErrorStats;

		const float ErrorThresholdToUse = ErrorThreshold / AnimSeq.ErrorThresholdScale;

		for (int32 ContextIndex = ComparisonLoopStart; ContextIndex < NumContextes; ++ContextIndex)
		{
			FAnimBoneCompressionContext* Context = ContextList[ContextIndex];

			if (!Context->bSuccess)
			{
				// Compression failed for this codec, ignore it
				continue;
			}

			const FAnimationErrorStats& ErrorStats = Context->Result.AnimData->BoneCompressionErrorStats;
			const int32 CurrentSize = Context->GetCompressedSize();
			const int32 MemorySavingsFromPrevious = BestSize - CurrentSize;

			const bool bLowersError = ErrorStats.MaxError < BestErrorStats.MaxError;
			const bool bErrorUnderThreshold = ErrorStats.MaxError <= ErrorThresholdToUse;

			/* or if it we want to force the error below the threshold and it reduces error */
			const bool bReducesErrorBelowThreshold = bLowersError && (BestErrorStats.MaxError > ErrorThresholdToUse) && bForceBelowThreshold;
			
			bool bKeepNewCompressionMethod = bReducesErrorBelowThreshold;

			/* or if has an acceptable error and saves space  */
			const bool bHasAcceptableErrorAndSavesSpace = bErrorUnderThreshold && (MemorySavingsFromPrevious > 0);
			bKeepNewCompressionMethod |= bHasAcceptableErrorAndSavesSpace;
			
			/* or if saves the same amount and an acceptable error that is lower than the previous best */
			const bool bLowersErrorAndSavesSameOrBetter = bErrorUnderThreshold && bLowersError && (MemorySavingsFromPrevious >= 0);
			bKeepNewCompressionMethod |= bLowersErrorAndSavesSameOrBetter;

			if (bKeepNewCompressionMethod)
			{
				BestContext = Context;
				BestSize = Context->GetCompressedSize();
				BestErrorStats = ErrorStats;
			}

			const int32 MemorySavingsFromOriginal = RawSize - CurrentSize;
			const FString CodecLabel = Context->Codec->Description.Len() == 0 ? Context->Codec->GetName() : Context->Codec->Description;
			UE_LOGF(LogAnimationCompression, Verbose, "- %ls - bytes saved(%i) from previous(%i) MaxError(%.2f) bLowersError(%d) %ls",
				*CodecLabel, MemorySavingsFromOriginal, MemorySavingsFromPrevious, ErrorStats.MaxError, bLowersError, bKeepNewCompressionMethod ? TEXT("(**Best so far**)") : TEXT(""));

			UE_LOGF(LogAnimationCompression, Verbose, "    bReducesErrorBelowThreshold(%d) bHasAcceptableErrorAndSavesSpace(%d) bLowersErrorAndSavesSameOrBetter(%d)",
				bReducesErrorBelowThreshold, bHasAcceptableErrorAndSavesSpace, bLowersErrorAndSavesSameOrBetter);

			UE_LOGF(LogAnimationCompression, Verbose, "    WinningCompressorError(%f) Settings ErrorThreshold(%f) ErrorThresholdToUse(%f) bForceBelowThreshold(%d) bErrorUnderThreshold(%d)",
				BestErrorStats.MaxError, ErrorThreshold, ErrorThresholdToUse, bForceBelowThreshold, bErrorUnderThreshold);
		}
	}

	const bool Success = BestContext != nullptr;

	if (Success)
	{
		// Move our compressed data and the relevant metadata for decompression
		OutCompressedData = MoveTemp(BestContext->Result);
	}

	// Cleanup
	for (FAnimBoneCompressionContext* Context : ContextList)
	{
		delete Context;
	}

	ContextList.Reset();

#if DEBUG_DUMP_ANIM_COMPRESSION_STATS
	const double CompressionEndTime = FPlatformTime::Seconds();
	const double CompressionElapsedTime = CompressionEndTime - CompressionStartTime;

	static double TotalCompressionTimeSecs = 0.0;
	TotalCompressionTimeSecs += CompressionElapsedTime;

	static int32 TotalCompressedSize = 0;
	TotalCompressedSize += OutCompressedData.AnimData->GetApproxCompressedSize();

	static int32 TotalNumSequencesCompressed = 0;
	TotalNumSequencesCompressed++;

	if (Success)
	{
		UE_LOGF(LogAnimationCompression, Warning, "Compression took %f seconds (total: %f seconds) (error: %f cm)", CompressionElapsedTime, TotalCompressionTimeSecs, OutCompressedData.AnimData->BoneCompressionErrorStats.MaxError);
		UE_LOGF(LogAnimationCompression, Warning, "Compression used codec '%ls' (%.2f MB for %d sequences)", *OutCompressedData.Codec->Description, double(TotalCompressedSize) / (1024.0 * 1024.0), TotalNumSequencesCompressed);
	}
	else
	{
		UE_LOGF(LogAnimationCompression, Warning, "Compression took %f seconds (total: %f seconds)", CompressionElapsedTime, TotalCompressionTimeSecs);
		UE_LOGF(LogAnimationCompression, Warning, "Compression failed");
	}
#endif

	return Success;
}

void UAnimBoneCompressionSettings::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Ar << ErrorThreshold;
	Ar << bForceBelowThreshold;

	int32 NumValidCodecs = 0;
	for (UAnimBoneCompressionCodec* Codec : Codecs)
	{
		if (Codec != nullptr)
		{
			Codec->PopulateDDCKey(KeyArgs, Ar);
			NumValidCodecs++;
		}
	}

	if (NumValidCodecs == 0)
	{
		static FString NoCodecString("<Missing Codec>");
		Ar << NoCodecString;
	}
}
#endif

