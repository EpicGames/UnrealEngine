// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimClassInterface.h"
#include "Animation/AnimNodeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimClassInterface)

const IAnimClassInterface* IAnimClassInterface::GetRootClass() const
{
	auto GetSuperClassInterface = [](const IAnimClassInterface* InClass) -> const IAnimClassInterface*
	{
		if(const UClass* ActualClass = GetActualAnimClass(InClass))
		{
			return GetFromClass(ActualClass->GetSuperClass());
		}

		return nullptr;
	};

	const IAnimClassInterface* RootClass = this;
	while(const IAnimClassInterface* NextClass = GetSuperClassInterface(RootClass))
	{
		RootClass = NextClass;
	}

	return RootClass;
}

IAnimClassInterface* IAnimClassInterface::GetFromClass(UClass* InClass)
{
	if (auto AnimClassInterface = Cast<IAnimClassInterface>(InClass))
	{
		return AnimClassInterface;
	}

	return nullptr;
}

const IAnimClassInterface* IAnimClassInterface::GetFromClass(const UClass* InClass)
{
	if (auto AnimClassInterface = Cast<const IAnimClassInterface>(InClass))
	{
		return AnimClassInterface;
	}

	return nullptr;
}

UClass* IAnimClassInterface::GetActualAnimClass(IAnimClassInterface* AnimClassInterface)
{
	if (UClass* ActualAnimClass = Cast<UClass>(AnimClassInterface))
	{
		return ActualAnimClass;
	}
	if (UObject* AsObject = Cast<UObject>(AnimClassInterface))
	{
		return Cast<UClass>(AsObject->GetOuter());
	}
	return nullptr;
}

const UClass* IAnimClassInterface::GetActualAnimClass(const IAnimClassInterface* AnimClassInterface)
{
	if (const UClass* ActualAnimClass = Cast<const UClass>(AnimClassInterface))
	{
		return ActualAnimClass;
	}
	if (const UObject* AsObject = Cast<const UObject>(AnimClassInterface))
	{
		return Cast<const UClass>(AsObject->GetOuter());
	}
	return nullptr;
}

const FAnimBlueprintFunction* IAnimClassInterface::FindAnimBlueprintFunction(IAnimClassInterface* AnimClassInterface, const FName& InFunctionName)
{
	for(const FAnimBlueprintFunction& Function : AnimClassInterface->GetAnimBlueprintFunctions())
	{
		if(Function.Name == InFunctionName)
		{
			return &Function;
		}
	}

	return nullptr;
}

bool IAnimClassInterface::IsAnimBlueprintFunction(IAnimClassInterface* InAnimClassInterface, const UFunction* InFunction)
{
	if(InFunction->GetOuterUClass() == GetActualAnimClass(InAnimClassInterface))
	{
		for(const FAnimBlueprintFunction& Function : InAnimClassInterface->GetAnimBlueprintFunctions())
		{
			if(Function.Name == InFunction->GetFName())
			{
				return true;
			}
		}
	}
	return false;
}

// This inverts the logic from FProperty::ContainerUObjectPtrToValuePtrInternal, allowing access to the containing UObject
// when supplied with one of its properties
static const UObject* ValuePtrToContainerUObjectPtr(FProperty* Property, const void* ValuePtr, int32 ArrayIndex)
{
	check(ArrayIndex < Property->ArrayDim);
	check(ValuePtr);

	const uint8* ContainerPtr = (const uint8*)ValuePtr - (Property->GetOffset_ForInternal() + Property->GetElementSize() * ArrayIndex);

	check(((const UObject*)ContainerPtr)->IsValidLowLevel()); // Check its a valid UObject that was passed in
	check(((const UObject*)ContainerPtr)->GetClass() != NULL);
	check(Property->GetOwner<UClass>()); // Check that the outer of this property is a UClass (not another property)

	// Check that the object we are accessing is of the class that contains this property
	checkf(((const UObject*)ContainerPtr)->IsA(Property->GetOwner<UClass>()), TEXT("'%s' is of class '%s' however property '%s' belongs to class '%s'")
		, *((const UObject*)ContainerPtr)->GetName()
		, *((const UObject*)ContainerPtr)->GetClass()->GetName()
		, *Property->GetName()
		, *(Property->GetOwner<UClass>())->GetName());

	return (const UObject*)ContainerPtr;
}

const UObject* IAnimClassInterface::GetObjectPtrFromAnimNode(const IAnimClassInterface* InAnimClassInterface, const FAnimNode_Base* InNode)
{
	const int32 NodeIndex = InNode->GetNodeIndex();
	FStructProperty* NodeProperty = InAnimClassInterface->GetAnimNodeProperties()[NodeIndex];
	return ValuePtrToContainerUObjectPtr(NodeProperty, InNode, 0);
}

const FAnimNode_Base* IAnimClassInterface::GetAnimNodeFromObjectPtr(const UObject* InObject, int32 InNodeIndex, UScriptStruct* InNodeType)
{
	if(IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(InObject->GetClass()))
	{
		check(AnimClassInterface->GetAnimNodeProperties()[InNodeIndex]);
		FStructProperty* NodeProperty = AnimClassInterface->GetAnimNodeProperties()[InNodeIndex];
		if(NodeProperty->Struct->IsChildOf(InNodeType))
		{
			return NodeProperty->ContainerPtrToValuePtr<const FAnimNode_Base>(InObject);
		}
	}

	return nullptr;
}

bool IAnimClassInterface::HasNodeAnyFlags(IAnimClassInterface* InAnimClassInterface, int32 InNodeIndex, EAnimNodeDataFlags InNodeDataFlags)
{
	const TArrayView<const FAnimNodeData> AnimNodeData = InAnimClassInterface->GetNodeData();
	check(AnimNodeData.IsValidIndex(InNodeIndex));
	return AnimNodeData[InNodeIndex].HasNodeAnyFlags(InNodeDataFlags);
}
