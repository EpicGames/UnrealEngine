// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimComposite.cpp: Composite classes that contains sequence for each section
=============================================================================*/ 

#include "Animation/AnimComposite.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AnimationPoseData.h"
#include "EngineLogs.h"
#include "Animation/AnimationSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimComposite)

UAnimComposite::UAnimComposite(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

#if WITH_EDITOR
bool UAnimComposite::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive /*= true*/) 
{
	return AnimationTrack.GetAllAnimationSequencesReferred(AnimationAssets, bRecursive);
}

void UAnimComposite::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	AnimationTrack.ReplaceReferredAnimations(ReplacementMap);
}

void UAnimComposite::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateCommonTargetFrameRate();
}
#endif

bool UAnimComposite::IsNotifyAvailable() const
{
	return (GetPlayLength() > 0.f && (Super::IsNotifyAvailable() || AnimationTrack.IsNotifyAvailable()));
}

void UAnimComposite::GetAnimNotifiesFromDeltaPositions(const float& PreviousPosition, const float & CurrentPosition, FAnimNotifyContext& NotifyContext) const
{
	Super::GetAnimNotifiesFromDeltaPositions(PreviousPosition, CurrentPosition, NotifyContext);
	AnimationTrack.GetAnimNotifiesFromTrackPositions(PreviousPosition, CurrentPosition, NotifyContext);
}

void UAnimComposite::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(AnimationTrack.GetTotalBytesUsed());
}


void UAnimComposite::HandleAssetPlayerTickedInternal(FAnimAssetTickContext &Context, const float PreviousTime, const float MoveDelta, const FAnimTickRecord &Instance, struct FAnimNotifyQueue& NotifyQueue) const
{
	Super::HandleAssetPlayerTickedInternal(Context, PreviousTime, MoveDelta, Instance, NotifyQueue);

	ExtractRootMotionFromTrack(AnimationTrack, PreviousTime, PreviousTime + MoveDelta, FAnimExtractContext(), Context.RootMotionMovementParams);
}

void UAnimComposite::GetAnimationPose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const
{
	AnimationTrack.GetAnimationPose(OutAnimationPoseData, ExtractionContext);

	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();

	FBlendedCurve CompositeCurve;
	CompositeCurve.InitFrom(OutCurve);
	EvaluateCurveData(CompositeCurve, ExtractionContext);

	// combine both curve
	OutCurve.Combine(CompositeCurve);
}

EAdditiveAnimationType UAnimComposite::GetAdditiveAnimType() const
{
	int32 AdditiveType = AnimationTrack.GetTrackAdditiveType();

	if (AdditiveType != -1)
	{
		return (EAdditiveAnimationType)AdditiveType;
	}

	return AAT_None;
}

void UAnimComposite::EnableRootMotionSettingFromMontage(bool bInEnableRootMotion, const ERootMotionRootLock::Type InRootMotionRootLock)
{
	AnimationTrack.EnableRootMotionSettingFromMontage(bInEnableRootMotion, InRootMotionRootLock);
}

bool UAnimComposite::HasRootMotion() const
{
	return AnimationTrack.HasRootMotion();
}

FTransform UAnimComposite::ExtractRootMotion(const FAnimExtractContext& ExtractionContext) const
{
	return ExtractRootMotionFromRange(ExtractionContext.CurrentTime, static_cast<double>(ExtractionContext.DeltaTimeRecord.Delta), ExtractionContext);
}

FTransform UAnimComposite::ExtractRootMotionFromRange(double StartTime, double EndTime, const FAnimExtractContext& ExtractionContext) const
{
	FRootMotionMovementParams RootMotion;
	ExtractRootMotionFromTrack(AnimationTrack, StartTime, EndTime, ExtractionContext, RootMotion);
	return RootMotion.GetRootMotionTransform();
}

FTransform UAnimComposite::ExtractRootTrackTransform(const FAnimExtractContext& ExtractionContext, const FBoneContainer* RequiredBones) const
{
	if (const FAnimSegment* AnimSegment = AnimationTrack.GetSegmentAtTime(ExtractionContext.CurrentTime))
	{
		float SegmentTime = 0.0f;
		if (const UAnimSequenceBase* SequenceBase = AnimSegment->GetAnimationData(ExtractionContext.CurrentTime, SegmentTime))
		{
			FAnimExtractContext Context = ExtractionContext;
			Context.CurrentTime = static_cast<double>(SegmentTime);
			return SequenceBase->ExtractRootTrackTransform(Context, RequiredBones);
		}
	}

	// Return the last valid value in case we're requesting for a time after the anim composite end time.
	if (!AnimationTrack.AnimSegments.IsEmpty())
	{
		const int32 LastSegmentIndex = AnimationTrack.AnimSegments.Num() - 1;
		if (ExtractionContext.CurrentTime > AnimationTrack.AnimSegments[LastSegmentIndex].AnimEndTime)
		{
			const UAnimSequenceBase* SequenceBase = AnimationTrack.AnimSegments[LastSegmentIndex].GetAnimReference().Get();

			FAnimExtractContext Context = ExtractionContext;
			Context.CurrentTime = static_cast<double>(SequenceBase->GetPlayLength());
			return SequenceBase->ExtractRootTrackTransform(Context, RequiredBones);
		}
	}

	return {};
}

#if WITH_EDITOR
class UAnimSequence* UAnimComposite::GetAdditiveBasePose() const
{
	// @todo : for now it just picks up the first sequence
	return AnimationTrack.GetAdditiveBasePose();
}
#endif 

void UAnimComposite::InvalidateRecursiveAsset()
{
	// unfortunately we'll have to do this all the time
	// we don't know whether or not the nested assets are modified or not
	AnimationTrack.InvalidateRecursiveAsset(this);
}

bool UAnimComposite::ContainRecursive(TArray<UAnimCompositeBase*>& CurrentAccumulatedList)
{
	// am I included already?
	if (CurrentAccumulatedList.Contains(this))
	{
		return true;
	}

	// otherwise, add myself to it
	CurrentAccumulatedList.Add(this);

	// otherwise send to animation track
	if (AnimationTrack.ContainRecursive(CurrentAccumulatedList))
	{
		return true;
	}

	return false;
}

void UAnimComposite::SetCompositeLength(float InLength)
{
#if WITH_EDITOR		
	Controller->SetNumberOfFrames(DataModelInterface->GetFrameRate().AsFrameNumber(InLength));
#else
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SequenceLength = InLength;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif	
}

void UAnimComposite::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	for (const FAnimSegment& AnimSegment : AnimationTrack.AnimSegments)
	{
		if(AnimSegment.IsPlayLengthOutOfDate())
		{
			UE_LOGF(LogAnimation, Warning, "AnimComposite (%ls) contains a Segment for which the playable length %f is out-of-sync with the represented AnimationSequence its length %f (%ls). Please up-date the segment and resave.", *GetFullName(), (AnimSegment.AnimEndTime - AnimSegment.AnimStartTime), AnimSegment.GetAnimReference()->GetPlayLength(), *AnimSegment.GetAnimReference()->GetFullName());
		}
	}
#endif
}

#if WITH_EDITOR
void UAnimComposite::UpdateCommonTargetFrameRate()
{
	CommonTargetFrameRate = FFrameRate(0,0);
	FFrameRate TargetRate = UAnimationSettings::Get()->GetDefaultFrameRate();

	bool bFirst = true;
	bool bValidFrameRate = AnimationTrack.AnimSegments.Num() != 0;
	for (const FAnimSegment& Segment : AnimationTrack.AnimSegments)
	{
		const UAnimSequenceBase* Base = Segment.GetAnimReference();
		if (Base && Base != this)
		{
			const FFrameRate BaseFrameRate = Base->GetSamplingFrameRate();
			if (bFirst)
			{
				TargetRate = BaseFrameRate;
				bFirst = false;
			}
			else
			{
				if (BaseFrameRate.IsValid())
				{					
					if (TargetRate.IsMultipleOf(BaseFrameRate))
					{
						TargetRate = BaseFrameRate;
					}
					else if (TargetRate != BaseFrameRate && !BaseFrameRate.IsMultipleOf(TargetRate))
					{						
						FString AssetString;
						TArray<UAnimationAsset*> Assets;
						if(GetAllAnimationSequencesReferred(Assets, false))
						{
							for (const UAnimationAsset* AnimAsset : Assets)
							{
								if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(AnimAsset))
								{
									AssetString.Append(FString::Printf(TEXT("\n\t%s - %s"), *AnimSequenceBase->GetName(), *AnimSequenceBase->GetSamplingFrameRate().ToPrettyText().ToString()));
								}
							}
						}						

						if (UE::Anim::CVarOutputMontageFrameRateWarning.GetValueOnAnyThread() == true)
						{
							UE_LOGF(LogAnimation, Warning, "Frame rate of animation %ls (%ls) is incompatible with other animations in Animation Composite %ls - underlying frame-rate will be set to %ls:%ls", *Base->GetName(), *BaseFrameRate.ToPrettyText().ToString(), *GetName(), *Super::GetSamplingFrameRate().ToPrettyText().ToString(), *AssetString);
						}
						
						bValidFrameRate = false;
						break;
					}
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Invalid frame rate %ls for %ls in %ls", *BaseFrameRate.ToPrettyText().ToString(), *Base->GetName(), *GetName());
				}
			}			
		}	
	}

	if (bValidFrameRate)
	{
		CommonTargetFrameRate = TargetRate;
	}
}
#endif // WITH_EDITOR

FFrameRate UAnimComposite::GetSamplingFrameRate() const
{
	if (CommonTargetFrameRate.IsValid())
	{
		return CommonTargetFrameRate;
	}

	return Super::GetSamplingFrameRate();
}

