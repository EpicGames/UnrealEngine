// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompositeBase.cpp: Anim Composite base class that contains AnimTrack data structure/interface
=============================================================================*/ 

#include "Animation/AnimCompositeBase.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimNotifyQueue.h"
#include "BonePose.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AttributesRuntime.h"
#include "EngineLogs.h"
#include "UObject/LinkerLoad.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCompositeBase)

#if WITH_EDITOR
namespace UE
{
	namespace Anim
	{		
		TAutoConsoleVariable<bool> CVarOutputMontageFrameRateWarning(
			TEXT("a.OutputMontageFrameRateWarning"),
			false,
			TEXT("If true will warn the user about Animation Montages/Composites composed of incompatible animation assets (incompatible frame-rates)."));
	}
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////
// FAnimSegment
///////////////////////////////////////////////////////

UAnimSequenceBase* FAnimSegment::GetAnimationData(float PositionInTrack, float& PositionInAnim) const
{
	if( bValid && IsInRange(PositionInTrack) )
	{
		if(AnimReference)
		{
			const float ValidPlayRate = GetValidPlayRate();

			// this result position should be pure position within animation
			float Delta = (PositionInTrack - StartPos);

			// LoopingCount should not be zero, and it should not get here, but just in case
			if (LoopingCount > 1)
			{
				// we need to consider looping count
				float AnimPlayLength = (AnimEndTime - AnimStartTime) / FMath::Abs(ValidPlayRate);
				Delta = FMath::Fmod(Delta, AnimPlayLength);
			}

			if (ValidPlayRate > 0.f)
			{
				PositionInAnim = AnimStartTime + Delta * ValidPlayRate;
			}
			else
			{
				PositionInAnim = AnimEndTime + Delta * ValidPlayRate;
			}

			return AnimReference;
		}
	}

	return nullptr;
}

/** Converts 'Track Position' to position on AnimSequence.
 * Note: doesn't check that position is in valid range, must do that before calling this function! */
float FAnimSegment::ConvertTrackPosToAnimPos(const float& TrackPosition) const
{
	const float PlayRate = GetValidPlayRate();
	const float AnimLength = (AnimEndTime - AnimStartTime);
	const float AnimPositionUnWrapped = (TrackPosition - StartPos) * PlayRate;

	// Figure out how many times animation is allowed to be looped.
	const float LoopCount = FMath::Min(FMath::FloorToInt(FMath::Abs(AnimPositionUnWrapped) / AnimLength), FMath::Max(LoopingCount-1, 0));
	// Position within AnimSequence
	const float AnimPoint = (PlayRate >= 0.f) ? AnimStartTime : AnimEndTime;

	const float AnimPosition = AnimPoint + (AnimPositionUnWrapped - float(LoopCount) * AnimLength);
	
	return AnimPosition;
}

void FAnimSegment::GetAnimNotifiesFromTrackPositions(const float& PreviousTrackPosition, const float& CurrentTrackPosition, FAnimNotifyContext& NotifyContext) const
{
	const bool bTrackPlayingBackwards = (PreviousTrackPosition > CurrentTrackPosition);
	const float SegmentStartPos = StartPos;
	const float SegmentEndPos = StartPos + GetLength();
	const bool bZeroTrackPositionDelta = CurrentTrackPosition == PreviousTrackPosition;

	// if track range overlaps segment
	if( bTrackPlayingBackwards 
		? ((CurrentTrackPosition < SegmentEndPos) && (PreviousTrackPosition > SegmentStartPos)) 
		: ((PreviousTrackPosition < SegmentEndPos) && (CurrentTrackPosition > SegmentStartPos)) 
		)
	{
		// Only allow AnimSequences for now. Other types will need additional support.
		UAnimSequenceBase* AnimSequenceBase = AnimReference;
		if(AnimSequenceBase)
		{
			const float ValidPlayRate = GetValidPlayRate();
			const float AbsValidPlayRate = FMath::Abs(ValidPlayRate);

			// Get starting position, closest overlap.
			float AnimStartPosition = ConvertTrackPosToAnimPos( bTrackPlayingBackwards ? FMath::Min(PreviousTrackPosition, SegmentEndPos) : FMath::Max(PreviousTrackPosition, SegmentStartPos) );
			AnimStartPosition = FMath::Clamp(AnimStartPosition, AnimStartTime, AnimEndTime);

			// When looping, the current track position could exceed the current segment (anim montage loops the track position after firing notifies)
			// We need to make sure to clamp the current/previous track positions within our segment
			float TrackTimeToGo = FMath::Abs(FMath::Clamp(CurrentTrackPosition, SegmentStartPos, SegmentEndPos) - FMath::Clamp(PreviousTrackPosition, SegmentStartPos, SegmentEndPos));

			// The track can be playing backwards and the animation can be playing backwards, so we
			// need to combine to work out what direction we are traveling through the animation
			bool bAnimPlayingBackwards = bTrackPlayingBackwards ^ (ValidPlayRate < 0.f);
			const float ResetStartPosition = bAnimPlayingBackwards ? AnimEndTime : AnimStartTime;

			// Abstract out end point since animation can be playing forward or backward.
			const float AnimEndPoint = bAnimPlayingBackwards ? AnimStartTime : AnimEndTime;

			for(int32 IterationsLeft=FMath::Max(LoopingCount, 1); ((IterationsLeft > 0) && (TrackTimeToGo > 0.f || bZeroTrackPositionDelta)); --IterationsLeft)
			{
				// Track time left to reach end point of animation.
				const float TrackTimeToAnimEndPoint = (AnimEndPoint - AnimStartPosition) / AbsValidPlayRate;

				// If our time left is shorter than time to end point, no problem. End there.
				// This will also run if we arrive with bZeroTrackPositionDelta == true, as TrackTimeToGo == 0.f
				if( FMath::Abs(TrackTimeToGo) < FMath::Abs(TrackTimeToAnimEndPoint) )
				{
					const float PlayRate = ValidPlayRate * (bTrackPlayingBackwards ? -1.f : 1.f);
					const float AnimEndPosition = (TrackTimeToGo * PlayRate) + AnimStartPosition;
					AnimSequenceBase->GetAnimNotifiesFromDeltaPositions(AnimStartPosition, AnimEndPosition, NotifyContext);
					break;
				}
				// Otherwise we hit the end point of the animation first...
				else
				{
					// Add that piece for extraction.
					AnimSequenceBase->GetAnimNotifiesFromDeltaPositions(AnimStartPosition, AnimEndPoint, NotifyContext);

					// decrease our TrackTimeToGo if we have to do another iteration.
					// and put ourselves back at the beginning of the animation.
					TrackTimeToGo -= TrackTimeToAnimEndPoint;
					AnimStartPosition = ResetStartPosition;
				}
			}
		}
	}
}

/** 
 * Given a Track delta position [StartTrackPosition, EndTrackPosition]
 * See if this AnimSegment overlaps any of it, and if it does, break them up into a sequence of FRootMotionExtractionStep.
 * Supports animation playing forward and backward. Track range should be a contiguous range, not wrapping over due to looping.
 */
void FAnimSegment::GetRootMotionExtractionStepsForTrackRange(TArray<FRootMotionExtractionStep> & RootMotionExtractionSteps, const float StartTrackPosition, const float EndTrackPosition) const
{
	if( StartTrackPosition == EndTrackPosition )
	{
		return;
	}

	if (!bValid || !AnimReference)
	{
		return;
	}

	const bool bTrackPlayingBackwards = (StartTrackPosition > EndTrackPosition);

	const float SegmentStartPos = StartPos;
	const float SegmentEndPos = StartPos + GetLength();

	// if range overlaps segment
	if (bTrackPlayingBackwards
		? ((EndTrackPosition < SegmentEndPos) && (StartTrackPosition > SegmentStartPos)) 
		: ((StartTrackPosition < SegmentEndPos) && (EndTrackPosition > SegmentStartPos)) 
		)
	{
		const float ValidPlayRate = GetValidPlayRate();
		const float AbsValidPlayRate = FMath::Abs(ValidPlayRate);

		const float StartTrackPositionForSegment = bTrackPlayingBackwards ? FMath::Min(StartTrackPosition, SegmentEndPos) : FMath::Max(StartTrackPosition, SegmentStartPos);
		const float EndTrackPositionForSegment = bTrackPlayingBackwards ? FMath::Max(EndTrackPosition, SegmentStartPos) : FMath::Min(EndTrackPosition, SegmentEndPos);

		// Get starting position, closest overlap.
		float AnimStartPosition = ConvertTrackPosToAnimPos(StartTrackPositionForSegment);
		AnimStartPosition = FMath::Clamp(AnimStartPosition, AnimStartTime, AnimEndTime);
		//check( (AnimStartPosition >= AnimStartTime) && (AnimStartPosition <= AnimEndTime) );
		float TrackTimeToGo = FMath::Abs(EndTrackPositionForSegment - StartTrackPositionForSegment);

		// The track can be playing backwards and the animation can be playing backwards, so we
		// need to combine to work out what direction we are traveling through the animation
		bool bAnimPlayingBackwards = bTrackPlayingBackwards ^ (ValidPlayRate < 0.f);
		const float ResetStartPosition = bAnimPlayingBackwards ? AnimEndTime : AnimStartTime;

		// Abstract out end point since animation can be playing forward or backward.
		const float AnimEndPoint = bAnimPlayingBackwards ? AnimStartTime : AnimEndTime;

		// Only allow AnimSequences for now. Other types will need additional support.
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimReference);
		UAnimComposite* AnimComposite = Cast<UAnimComposite>(AnimReference);

		if (AnimSequence || AnimComposite)
		{
			for(int32 IterationsLeft=FMath::Max(LoopingCount, 1); ((IterationsLeft > 0) && (TrackTimeToGo > 0.f)); --IterationsLeft)
			{
				// Track time left to reach end point of animation.
				const float TrackTimeToAnimEndPoint = (AnimEndPoint - AnimStartPosition) / ValidPlayRate;

				// If our time left is shorter than time to end point, no problem. End there.
				if( FMath::Abs(TrackTimeToGo) < FMath::Abs(TrackTimeToAnimEndPoint) )
				{
					const float PlayRate = ValidPlayRate * (bTrackPlayingBackwards ? -1.f : 1.f);
					const float AnimEndPosition = (TrackTimeToGo * PlayRate) + AnimStartPosition;
					if (AnimSequence)
					{
						RootMotionExtractionSteps.Add(FRootMotionExtractionStep(AnimSequence, AnimStartPosition, AnimEndPosition));
					}
					else if (AnimComposite)
					{
						AnimComposite->AnimationTrack.GetRootMotionExtractionStepsForTrackRange(RootMotionExtractionSteps, AnimStartPosition, AnimEndPosition);
					}
					break;
				}
				// Otherwise we hit the end point of the animation first...
				else
				{
					// Add that piece for extraction.
					if (AnimSequence)
					{
						RootMotionExtractionSteps.Add(FRootMotionExtractionStep(AnimSequence, AnimStartPosition, AnimEndPoint));
					}
					else if (AnimComposite)
					{
						AnimComposite->AnimationTrack.GetRootMotionExtractionStepsForTrackRange(RootMotionExtractionSteps, AnimStartPosition, AnimEndPoint);
					}

					// decrease our TrackTimeToGo if we have to do another iteration.
					// and put ourselves back at the beginning of the animation.
					TrackTimeToGo -= TrackTimeToAnimEndPoint;
					AnimStartPosition = ResetStartPosition;
				}
			}
		}
	}
}

///////////////////////////////////////////////////////
// FAnimTrack
///////////////////////////////////////////////////////
bool FAnimTrack::HasRootMotion() const
{
	for (const FAnimSegment& AnimSegment : AnimSegments)
	{
		const UAnimSequenceBase* AnimReference = AnimSegment.GetAnimReference();
		if (AnimSegment.bValid && AnimReference && AnimReference->HasRootMotion())
		{
			return true;
		}
	}
	return false;
}

#if WITH_EDITOR
class UAnimSequence* FAnimTrack::GetAdditiveBasePose() const
{
	if (IsAdditive())
	{
		for (const FAnimSegment& AnimSegment : AnimSegments)
		{
			UAnimSequenceBase* AnimReference = AnimSegment.GetAnimReference();
			UAnimSequence* BasePose = AnimReference ? AnimReference->GetAdditiveBasePose() : nullptr;
			if (BasePose)
			{
				return BasePose;
			}
		}
	}
	return nullptr;
}
#endif

/** 
 * Given a Track delta position [StartTrackPosition, EndTrackPosition]
 * See if any AnimSegment overlaps any of it, and if any do, break them up into a sequence of FRootMotionExtractionStep.
 * Supports animation playing forward and backward. Track range should be a contiguous range, not wrapping over due to looping.
 */
void FAnimTrack::GetRootMotionExtractionStepsForTrackRange(TArray<FRootMotionExtractionStep> & RootMotionExtractionSteps, const float StartTrackPosition, const float EndTrackPosition) const
{
	// must extract root motion in right order sequentially
	const bool bPlayingBackwards = (StartTrackPosition > EndTrackPosition);
	if( bPlayingBackwards )
	{
		for(int32 AnimSegmentIndex=AnimSegments.Num()-1; AnimSegmentIndex>=0; AnimSegmentIndex--)
		{
			const FAnimSegment& AnimSegment = AnimSegments[AnimSegmentIndex];
			AnimSegment.GetRootMotionExtractionStepsForTrackRange(RootMotionExtractionSteps, StartTrackPosition, EndTrackPosition);
		}
	}
	else
	{
		for(int32 AnimSegmentIndex=0; AnimSegmentIndex<AnimSegments.Num(); AnimSegmentIndex++)
		{
			const FAnimSegment& AnimSegment = AnimSegments[AnimSegmentIndex];
			AnimSegment.GetRootMotionExtractionStepsForTrackRange(RootMotionExtractionSteps, StartTrackPosition, EndTrackPosition);
		}
	}
}

float FAnimTrack::GetLength() const
{
	float TotalLength = 0.f;

	// in the future, if we're more clear about exactly what requirement is for segments, 
	// this can be optimized. For now this is slow. 
	for (const FAnimSegment& AnimSegment : AnimSegments)
	{
		const float EndFrame = AnimSegment.StartPos + AnimSegment.GetLength();
		if ( EndFrame > TotalLength )
		{
			TotalLength = EndFrame;
		}
	}

	return TotalLength;
}

bool FAnimTrack::IsAdditive() const
{
	// this function just checks first animation to verify if this is additive or not
	// if first one is additive, it returns true, 
	// the best way to handle isn't really practical. If I do go through all of them
	// and if they mismatch, what can I do? That should be another verification function when this is created
	// it will look visually wrong if something mismatches, but nothing really is better solution than that. 
	// in editor, when this is created, the test has to be done to verify all are matches. 
	if (AnimSegments.Num() > 0)
	{
		const FAnimSegment& AnimSegment = AnimSegments[0];
		const UAnimSequenceBase* AnimReference = AnimSegment.GetAnimReference();
		return (AnimReference && AnimSegment.bValid && AnimReference->IsValidAdditive() );
	}

	return false;
}

bool FAnimTrack::IsRotationOffsetAdditive() const
{
	// this function just checks first animation to verify if this is additive or not
	// if first one is additive, it returns true, 
	// the best way to handle isn't really practical. If I do go through all of them
	// and if they mismatch, what can I do? That should be another verification function when this is created
	// it will look visually wrong if something mismatches, but nothing really is better solution than that. 
	// in editor, when this is created, the test has to be done to verify all are matches. 
	if (AnimSegments.Num() > 0)
	{
		const FAnimSegment& AnimSegment = AnimSegments[0];
		const UAnimSequenceBase* AnimReference = AnimSegment.GetAnimReference();
		if (AnimReference && AnimReference->IsValidAdditive())
		{
			return (AnimReference->GetAdditiveAnimType() == AAT_RotationOffsetMeshSpace);
		}
	}

	return false;
}

int32 FAnimTrack::GetTrackAdditiveType() const
{
	// this function just checks first animation to verify the type
	// the best way to handle isn't really practical. If I do go through all of them
	// and if they mismatch, what can I do? That should be another verification function when this is created
	// it will look visually wrong if something mismatches, but nothing really is better solution than that. 
	// in editor, when this is created, the test has to be done to verify all are matches. 

	if( AnimSegments.Num() > 0 )
	{
		const FAnimSegment& AnimSegment = AnimSegments[0];
		if (const UAnimSequenceBase* AnimReference = AnimSegment.GetAnimReference())
		{
			return AnimReference->GetAdditiveAnimType();
		}
	}
	return -1;
}

void FAnimTrack::ValidateSegmentTimes()
{
	// rearrange, make sure there are no gaps between and all start times are correctly set
	if(AnimSegments.Num() > 0)
	{
		AnimSegments[0].StartPos = 0.0f;
		for(int32 SegmentIndex = 0; SegmentIndex < AnimSegments.Num(); SegmentIndex++)
		{
			FAnimSegment& AnimSegment = AnimSegments[SegmentIndex];
			if(SegmentIndex > 0)
			{
				AnimSegment.StartPos = AnimSegments[SegmentIndex - 1].StartPos + AnimSegments[SegmentIndex - 1].GetLength();
			}

			const UAnimSequenceBase* AnimReference = AnimSegment.GetAnimReference();
			if(AnimReference && AnimSegment.AnimEndTime > AnimReference->GetPlayLength())
			{
				AnimSegment.AnimEndTime = AnimReference->GetPlayLength();
			}
		}
	}
}

FAnimSegment* FAnimTrack::GetSegmentAtTime(float InTime)
{
	const int32 SegmentIndex = GetSegmentIndexAtTime(InTime);
	return (SegmentIndex != INDEX_NONE) ? &(AnimSegments[SegmentIndex]) : nullptr;
}

const FAnimSegment* FAnimTrack::GetSegmentAtTime(float InTime) const
{
	const int32 SegmentIndex = GetSegmentIndexAtTime(InTime);
	return (SegmentIndex != INDEX_NONE) ? &(AnimSegments[SegmentIndex]) : nullptr;
}

int32 FAnimTrack::GetSegmentIndexAtTime(float InTime) const
{
	// Montage Segments overlap on a single frame.
	// So last frame of Segment1 overlaps first frame of Segment2.
	// But in that case we want Segment2 to win.
	// So we iterate through these segments in reverse 
	// and return the first match with an inclusive range check.
	for(int32 Idx = AnimSegments.Num()-1; Idx >= 0; Idx--)
	{
		const FAnimSegment& Segment = AnimSegments[Idx];
		if (Segment.IsInRange(InTime))
		{
			return Idx;
		}
	}
	return INDEX_NONE;
}

#if WITH_EDITOR
bool FAnimTrack::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive/* = true*/) const
{
	for (const FAnimSegment& AnimSegment : AnimSegments)
	{
		UAnimSequenceBase* AnimSeqBase = AnimSegment.GetAnimReference();
		if ( AnimSegment.bValid && AnimSeqBase )
		{
			AnimSeqBase->HandleAnimReferenceCollection(AnimationAssets, bRecursive);
		}
	}

	return (AnimationAssets.Num() > 0 );
}

void FAnimTrack::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	TArray<FAnimSegment> NewAnimSegments;
	for (FAnimSegment& AnimSegment : AnimSegments)
	{
		if (AnimSegment.IsValid())
		{
			if (UAnimSequenceBase* SequenceBase = AnimSegment.GetAnimReference())
			{
				if(UAnimationAsset* const* ReplacementAsset = ReplacementMap.Find(SequenceBase))
				{
					AnimSegment.SetAnimReference(Cast<UAnimSequenceBase>(*ReplacementAsset));
					NewAnimSegments.Add(AnimSegment);
				}

				SequenceBase->ReplaceReferredAnimations(ReplacementMap);
			}
		}
	}

	AnimSegments = NewAnimSegments;
}

void FAnimTrack::CollapseAnimSegments()
{
	if(AnimSegments.Num() > 0)
	{
		// Sort function
		struct FSortFloatInt
		{
			bool operator()( const TKeyValuePair<float, int32> &A, const TKeyValuePair<float, int32>&B ) const
			{
				return A.Key < B.Key;
			}
		};

		// Create sorted map of start time to segment
		TArray<TKeyValuePair<float, int32>> m;
		for( int32 SegmentInd=0; SegmentInd < AnimSegments.Num(); ++SegmentInd )
		{
			m.Add(TKeyValuePair<float, int32>(AnimSegments[SegmentInd].StartPos, SegmentInd));
		}
		m.Sort(FSortFloatInt());

		//collapse all start times based on sorted map
		FAnimSegment* PrevSegment = &AnimSegments[m[0].Value];
		PrevSegment->StartPos = 0.0f;

		for ( int32 SegmentInd=1; SegmentInd < m.Num(); ++SegmentInd )
		{
			FAnimSegment* CurrSegment = &AnimSegments[m[SegmentInd].Value];
			CurrSegment->StartPos = PrevSegment->StartPos + PrevSegment->GetLength();
			PrevSegment = CurrSegment;
		}
	}
}

void FAnimTrack::SortAnimSegments()
{
	if(AnimSegments.Num() > 0)
	{
		struct FCompareSegments
		{
			bool operator()( const FAnimSegment &A, const FAnimSegment&B ) const
			{
				return A.StartPos < B.StartPos;
			}
		};

		AnimSegments.Sort( FCompareSegments() );

		ValidateSegmentTimes();
	}
}
#endif

void FAnimTrack::GetAnimationPose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const
{
	bool bExtractedPose = false;
	const float ClampedTime = FMath::Clamp(ExtractionContext.CurrentTime, 0.f, GetLength());

	if (const FAnimSegment* const AnimSegment = GetSegmentAtTime(ClampedTime))
	{
		if (AnimSegment->bValid)
		{
			// Copy passed in Extraction Context, but override position and root motion parameters.
			float PositionInAnim = 0.f;
			if (const UAnimSequenceBase* const AnimRef = AnimSegment->GetAnimationData(ClampedTime, PositionInAnim))
			{
				FAnimExtractContext SequenceExtractionContext(ExtractionContext);
				SequenceExtractionContext.CurrentTime = static_cast<double>(PositionInAnim);
				SequenceExtractionContext.DeltaTimeRecord.SetPrevious(
					SequenceExtractionContext.CurrentTime - SequenceExtractionContext.DeltaTimeRecord.Delta);
				SequenceExtractionContext.bExtractRootMotion &= AnimRef->HasRootMotion();
				SequenceExtractionContext.bLooping = AnimSegment->LoopingCount > 1;
				AnimRef->GetAnimationPose(OutAnimationPoseData, SequenceExtractionContext);
				bExtractedPose = true;
			}
		}
	}

	if (!bExtractedPose)
	{
		OutAnimationPoseData.GetPose().ResetToRefPose();
	}
}

void FAnimTrack::EnableRootMotionSettingFromMontage(bool bInEnableRootMotion, const ERootMotionRootLock::Type InRootMotionRootLock)
{
	for (const FAnimSegment& AnimSegment : AnimSegments)
	{
		if (UAnimSequenceBase* AnimReference = AnimSegment.GetAnimReference())
		{
			AnimReference->EnableRootMotionSettingFromMontage(bInEnableRootMotion, InRootMotionRootLock);
		}
	}
}

// this is to prevent anybody adding recursive asset to anim composite
// as a result of anim composite being a part of anim sequence base
void FAnimTrack::InvalidateRecursiveAsset(class UAnimCompositeBase* CheckAsset)
{
	for (FAnimSegment& AnimSegment : AnimSegments)
	{
		UAnimSequenceBase* SequenceBase = AnimSegment.GetAnimReference();
		UAnimCompositeBase* CompositeBase = Cast<UAnimCompositeBase>(SequenceBase);
		if (CompositeBase)
		{
			// add owner
			TArray<UAnimCompositeBase*> CompositeBaseRecurisve;
			CompositeBaseRecurisve.Add(CheckAsset);

			if (CompositeBase->ContainRecursive(CompositeBaseRecurisve))
			{
				AnimSegment.bValid = false;
			}
			else
			{
				AnimSegment.bValid = IsValidToAdd(CompositeBase);
			}
		}
		else
		{
			AnimSegment.bValid = IsValidToAdd(SequenceBase);
		}
	}
}

// this is recursive function that look thorough internal assets 
// and return true if it finds nested same assets
bool FAnimTrack::ContainRecursive(const TArray<UAnimCompositeBase*>& CurrentAccumulatedList)
{
	for (const FAnimSegment& AnimSegment : AnimSegments)
	{
		// we don't want to send this list broad widely (but in depth search)
		// to do that, we copy the current accumulated list, and send that only, not the siblings
		TArray<UAnimCompositeBase*> LocalCurrentAccumulatedList = CurrentAccumulatedList;
		UAnimCompositeBase* CompositeBase = Cast<UAnimCompositeBase>(AnimSegment.GetAnimReference());
		if (CompositeBase && CompositeBase->ContainRecursive(LocalCurrentAccumulatedList))
		{
			return true;
		}
	}

	return false;
}

void FAnimTrack::GetAnimNotifiesFromTrackPositions(const float& PreviousTrackPosition, const float& CurrentTrackPosition, FAnimNotifyContext& NotifyContext) const
{
	for (int32 SegmentIndex = 0; SegmentIndex<AnimSegments.Num(); ++SegmentIndex)
	{
		if (AnimSegments[SegmentIndex].IsValid())
		{
			AnimSegments[SegmentIndex].GetAnimNotifiesFromTrackPositions(PreviousTrackPosition, CurrentTrackPosition, NotifyContext);
		}
	}
}

bool FAnimTrack::IsNotifyAvailable() const
{
	for (int32 SegmentIndex = 0; SegmentIndex < AnimSegments.Num(); ++SegmentIndex)
	{
		if (AnimSegments[SegmentIndex].IsNotifyAvailable())
		{
			return true;
		}
	}

	return false;
}

int32 FAnimTrack::GetTotalBytesUsed() const
{
	return AnimSegments.GetAllocatedSize();
}

bool FAnimTrack::IsValidToAdd(const UAnimSequenceBase* SequenceBase, FText* OutReason /*= nullptr*/) const
{
	bool bValid = false;
	// remove asset if invalid
	if (SequenceBase)
	{
		const float PlayLength = SequenceBase->GetPlayLength();
		if (PlayLength <= 0.f)
		{
			UE_LOGF(LogAnimation, Warning, "Remove Empty Sequence (%ls)", *SequenceBase->GetFullName());

			if (OutReason)
			{
				*OutReason = FText::FromString(FString::Printf(TEXT("Animation Asset %s has invalid playable length of %f"), *SequenceBase->GetName(), PlayLength));
			}			
			
			return false;
		}

		if (!SequenceBase->CanBeUsedInComposition())
		{
			UE_LOGF(LogAnimation, Warning, "Remove Invalid Sequence (%ls)", *SequenceBase->GetFullName());
			if (OutReason)
			{
				*OutReason = FText::FromString(FString::Printf(TEXT("Animation Asset %s cannot be used in an Animation Composite/Montage"), *SequenceBase->GetName()));
			}
			return false;
		}
		
		const int32 TrackType = GetTrackAdditiveType();
		const EAdditiveAnimationType AnimAdditiveType = SequenceBase->GetAdditiveAnimType();
		if (TrackType != AnimAdditiveType && TrackType != INDEX_NONE)
		{
			const UEnum* TypeEnum = FindObject<UEnum>(nullptr, TEXT("/Script/Engine.EAdditiveAnimationType"));	
			if (OutReason)
			{
				*OutReason = FText::FromString(FString::Printf(TEXT("Animation Asset %s has an additive type %s that does not match the target's %s"), *SequenceBase->GetName(), *TypeEnum->GetNameStringByValue(AnimAdditiveType), *TypeEnum->GetNameStringByValue(TrackType)));
			}
			return false;
		}
		
		return true;
	}

	return true;
}
///////////////////////////////////////////////////////
// UAnimCompositeBase
///////////////////////////////////////////////////////

UAnimCompositeBase::UAnimCompositeBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimCompositeBase::ExtractRootMotionFromTrack(const FAnimTrack& SlotAnimTrack, float StartTrackPosition, float EndTrackPosition, const FAnimExtractContext& Context, FRootMotionMovementParams& RootMotion) const
{
	TArray<FRootMotionExtractionStep> RootMotionExtractionSteps;
	SlotAnimTrack.GetRootMotionExtractionStepsForTrackRange(RootMotionExtractionSteps, StartTrackPosition, EndTrackPosition);

	UE_LOGF(LogRootMotion, Verbose, "\tUAnimCompositeBase::ExtractRootMotionFromTrack, NumSteps: %d, StartTrackPosition: %.3f, EndTrackPosition: %.3f",
		RootMotionExtractionSteps.Num(), StartTrackPosition, EndTrackPosition);

	// Go through steps sequentially, extract root motion, and accumulate it.
	// This has to be done in order so root motion translation & rotation is applied properly (as translation is relative to rotation)
	for (int32 StepIndex = 0; StepIndex < RootMotionExtractionSteps.Num(); StepIndex++)
	{
		const FRootMotionExtractionStep & CurrentStep = RootMotionExtractionSteps[StepIndex];
		if (CurrentStep.AnimSequence->bEnableRootMotion)
		{
			FTransform DeltaTransform = CurrentStep.AnimSequence->ExtractRootMotionFromRange(CurrentStep.StartPosition, CurrentStep.EndPosition, Context);
			RootMotion.Accumulate(DeltaTransform);
		
			UE_LOGF(LogRootMotion, Log, "\t\tCurrentStep: %d, StartPos: %.3f, EndPos: %.3f, Anim: %ls DeltaTransform Translation: %ls, Rotation: %ls",
				StepIndex, CurrentStep.StartPosition, CurrentStep.EndPosition, *CurrentStep.AnimSequence->GetName(),
				*DeltaTransform.GetTranslation().ToCompactString(), *DeltaTransform.GetRotation().Rotator().ToCompactString());
		}
	}
}

FFrameRate UAnimCompositeBase::GetSamplingFrameRate() const
{
	// Allowing for 0.00001s precision in composite/montage length
	static const FFrameRate CompositeFrameRate(100000, 1);
	return CompositeFrameRate;
}

void UAnimCompositeBase::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	UpdateCommonTargetFrameRate();
#endif // WITH_EDITOR

	InvalidateRecursiveAsset();
}

void FAnimSegment::SetAnimReference(UAnimSequenceBase* InAnimReference, bool bInitialize /*= false*/)
{
	AnimReference = InAnimReference;

#if WITH_EDITOR
	UpdateCachedPlayLength();
#endif // WITH_EDITOR

	if (AnimReference && bInitialize)
	{		
		AnimStartTime = 0.f;
		AnimEndTime = AnimReference->GetPlayLength();
		AnimPlayRate = 1.f;
		LoopingCount = 1;
		StartPos = 0.f;
	}
}

#if WITH_EDITOR
bool FAnimSegment::IsPlayLengthOutOfDate() const
{
	if (AnimReference && !FMath::IsNearlyZero(CachedPlayLength))
	{
		// When the segment length is equal to _cached_ playlength and the current model playlength is different flag as out-of-date
		// this can happen when the sequence is reimported without updating the montage and thus ending up with 'invalid' playback range.
		const float PlayableLength = (AnimEndTime - AnimStartTime);
		return FMath::IsNearlyEqual(PlayableLength, CachedPlayLength, UE_KINDA_SMALL_NUMBER) && !FMath::IsNearlyEqual(AnimReference->GetPlayLength(), CachedPlayLength, UE_KINDA_SMALL_NUMBER);
	}
	
	return false;
}

void FAnimSegment::UpdateCachedPlayLength()
{
	CachedPlayLength = 0.f;
	const IAnimationDataModel* DataModel = AnimReference ? AnimReference->GetDataModel() : nullptr;	
	if(DataModel)
	{
		CachedPlayLength = DataModel->GetPlayLength();
	}
}

void UAnimCompositeBase::PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> ExistingDataModel)
{
	Super::PopulateWithExistingModel(ExistingDataModel);
	Controller->SetFrameRate(GetSamplingFrameRate());
}
#endif // WITH_EDITOR