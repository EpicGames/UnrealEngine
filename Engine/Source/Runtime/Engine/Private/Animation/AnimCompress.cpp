// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompress.cpp: Skeletal mesh animation compression.
=============================================================================*/ 

#include "Animation/AnimCompress.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Misc/MessageDialog.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "Misc/FeedbackContext.h"
#include "AnimationUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCompress)

DEFINE_LOG_CATEGORY(LogAnimationCompression);

#if WITH_EDITOR
namespace UE
{
	namespace Anim
	{
		namespace Compression
		{
			std::atomic<bool> FAnimationCompressionMemorySummaryScope::ScopeExists = false;
			TUniquePtr<FCompressionMemorySummary> FAnimationCompressionMemorySummaryScope::CompressionSummary = nullptr;
		}
	}
}
#endif // WITH_EDITOR

UAnimCompress::UAnimCompress(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TranslationCompressionFormat = ACF_None;
	RotationCompressionFormat = ACF_Float96NoW;
}

void FCompressionMemorySummary::GatherPostCompressionStats(const FCompressedAnimSequence& CompressedData, const TArray<FBoneData>& BoneData, const FName AnimFName, double CompressionTime, bool bInPerformedCompression)
{
#if WITH_EDITORONLY_DATA
	if (bEnabled && CompressedData.CompressedDataStructure)
	{
		bPerformedCompression |= bInPerformedCompression;

		TotalAfterCompressed +=  CompressedData.CompressedDataStructure->GetApproxCompressedSize();
		TotalCompressionExecutionTime += CompressionTime;

		const AnimationErrorStats& ErrorStats = CompressedData.CompressedDataStructure->BoneCompressionErrorStats;

		ErrorTotal += ErrorStats.AverageError;
		ErrorCount += 1.0f;
		AverageError = ErrorTotal / ErrorCount;

		WorstBoneError.StoreErrorStat(ErrorStats.MaxError, ErrorStats.MaxError, ErrorStats.MaxErrorTime, ErrorStats.MaxErrorBone, BoneData[ErrorStats.MaxErrorBone].Name, AnimFName);

		WorstAnimationError.StoreErrorStat(ErrorStats.AverageError, ErrorStats.AverageError, AnimFName);
	}
#endif
}


FCompressionMemorySummary::FCompressionMemorySummary(bool bInEnabled)
	: bEnabled(bInEnabled)
	, bUsed(false)
	, bPerformedCompression(false)
	, TotalRaw(0)
	, TotalBeforeCompressed(0)
	, TotalAfterCompressed(0)
	, NumberOfAnimations(0)
	, TotalCompressionExecutionTime(0.0)
	, ErrorTotal(0)
	, ErrorCount(0)
	, AverageError(0)
{
}

FCompressionMemorySummary::~FCompressionMemorySummary()
{
	if (bEnabled && bUsed)
	{
		const int32 TotalBeforeSaving = TotalRaw - TotalBeforeCompressed;
		const int32 TotalAfterSaving = TotalRaw - TotalAfterCompressed;
		const float OldCompressionRatio = (TotalBeforeCompressed > 0.f) ? (static_cast<float>(TotalRaw) / TotalBeforeCompressed) : 0.f;
		const float NewCompressionRatio = (TotalAfterCompressed > 0.f) ? (static_cast<float>(TotalRaw) / TotalAfterCompressed) : 0.f;

		FNumberFormattingOptions Options;
		Options.MinimumIntegralDigits = 7;
		Options.MinimumFractionalDigits = 2;

		FFormatNamedArguments Args;
		Args.Add(TEXT("TotalRaw"), FText::AsMemory(TotalRaw, &Options));
		Args.Add(TEXT("TotalBeforeCompressed"), FText::AsMemory(TotalBeforeCompressed, &Options));
		Args.Add(TEXT("TotalBeforeSaving"), FText::AsMemory(TotalBeforeSaving, &Options));
		Args.Add(TEXT("NumberOfAnimations"), FText::AsNumber(NumberOfAnimations));
		Args.Add(TEXT("OldCompressionRatio"), OldCompressionRatio);

		Args.Add(TEXT("TotalAfterCompressed"), FText::AsMemory(TotalAfterCompressed, &Options));
		Args.Add(TEXT("TotalAfterSaving"), FText::AsMemory(TotalAfterSaving, &Options));
		Args.Add(TEXT("NewCompressionRatio"), NewCompressionRatio);
		Args.Add(TEXT("TotalTimeSpentCompressingPretty"), FText::FromString(FPlatformTime::PrettyTime(TotalCompressionExecutionTime)));
		Args.Add(TEXT("TotalTimeSpentCompressingRawSeconds"), FText::AsNumber((float)TotalCompressionExecutionTime, &Options));

		Args.Add(TEXT("AverageError"), FText::AsNumber(AverageError, &Options));
		if (WorstBoneError.IsValid())
		{
			const FErrorTrackerWorstBone WorstBone = WorstBoneError.GetMaxErrorItem();
			const FErrorTrackerWorstAnimation WorstAnimation = WorstAnimationError.GetMaxErrorItem();
			Args.Add(TEXT("WorstBoneError"), WorstBone.ToText());
			Args.Add(TEXT("WorstAnimationError"), WorstAnimation.ToText());
		}
		else
		{
			Args.Add(TEXT("WorstBoneError"), FText::FromName(NAME_None));
			Args.Add(TEXT("WorstAnimationError"), FText::FromName(NAME_None));
		}

		FText Message;
		if (!bPerformedCompression)
		{
			Message = FText::Format(NSLOCTEXT("Engine", "CompressionMemorySummaryNoChange", "Fetched compressed data for {NumberOfAnimations} Animation(s)\n\nRaw: {TotalRaw} - Compressed: {TotalAfterCompressed}\nSaving: {TotalAfterSaving} ({NewCompressionRatio})\n\nEnd Effector Translation Added By Compression:\n Average: {AverageError} Max:\n{WorstBoneError}\n\nWorst Animation Error Seen:\n{WorstAnimationError}"), Args);
		}
		else
		{
			Message = FText::Format(NSLOCTEXT("Engine", "CompressionMemorySummary", "Compressed {NumberOfAnimations} Animation(s)\n\nPre Compression:\n\nRaw: {TotalRaw} - Compressed: {TotalBeforeCompressed}\nSaving: {TotalBeforeSaving} ({OldCompressionRatio})\n\nPost Compression:\n\nRaw: {TotalRaw} - Compressed: {TotalAfterCompressed}\nSaving: {TotalAfterSaving} ({NewCompressionRatio})\n\nTotal Compression Time: {TotalTimeSpentCompressingPretty} (Seconds: {TotalTimeSpentCompressingRawSeconds})\n\nEnd Effector Translation Added By Compression:\n Average: {AverageError} Max:\n{WorstBoneError}\n\nWorst Animation Error Seen:\n{WorstAnimationError}"), Args);
		}

		UE_LOGF(LogAnimationCompression, Display, "Top 10 Worst Bone Errors:");
		WorstBoneError.LogErrorStat();
		UE_LOGF(LogAnimationCompression, Display, "Top 10 Worst Average Animation Errors:");
		WorstAnimationError.LogErrorStat();
		FMessageDialog::Open(EAppMsgType::Ok, Message);
	}
}

#if WITH_EDITOR

void UAnimCompress::UnalignedWriteToStream(TArray<uint8>& ByteStream, const void* Src, SIZE_T Len)
{
	const int32 Offset = ByteStream.AddUninitialized(Len);
	FMemory::Memcpy(ByteStream.GetData() + Offset, Src, Len);
}

void UAnimCompress::UnalignedWriteToStream(TArray<uint8>& ByteStream, int32& StreamOffset, const void* Src, SIZE_T Len)
{
	FMemory::Memcpy(ByteStream.GetData() + StreamOffset, Src, Len);
	StreamOffset += static_cast<int32>(Len);
}


void UAnimCompress::PackVectorToStream(
	TArray<uint8>& ByteStream,
	AnimationCompressionFormat Format,
	const FVector3f& Vec,
	const float* Mins,
	const float* Ranges)
{
	if ( Format == ACF_None )
	{
		UnalignedWriteToStream( ByteStream, &Vec, sizeof(Vec) );
	}
	else if ( Format == ACF_Float96NoW )
	{
		UnalignedWriteToStream( ByteStream, &Vec, sizeof(Vec) );
	}
	else if ( Format == ACF_IntervalFixed32NoW )
	{
		const FVectorIntervalFixed32NoW CompressedVec( Vec, Mins, Ranges );

		UnalignedWriteToStream( ByteStream, &CompressedVec, sizeof(CompressedVec) );
	}
}

void UAnimCompress::PackQuaternionToStream(
	TArray<uint8>& ByteStream,
	AnimationCompressionFormat Format,
	const FQuat4f& Quat,
	const float* Mins,
	const float* Ranges)
{
	if ( Format == ACF_None )
	{
		UnalignedWriteToStream( ByteStream, &Quat, sizeof(Quat) );
	}
	else if ( Format == ACF_Float96NoW )
	{
		const FQuatFloat96NoW QuatFloat96NoW( Quat );
		UnalignedWriteToStream( ByteStream, &QuatFloat96NoW, sizeof(FQuatFloat96NoW) );
	}
	else if ( Format == ACF_Fixed32NoW )
	{
		const FQuatFixed32NoW QuatFixed32NoW( Quat );
		UnalignedWriteToStream( ByteStream, &QuatFixed32NoW, sizeof(FQuatFixed32NoW) );
	}
	else if ( Format == ACF_Fixed48NoW )
	{
		const FQuatFixed48NoW QuatFixed48NoW( Quat );
		UnalignedWriteToStream( ByteStream, &QuatFixed48NoW, sizeof(FQuatFixed48NoW) );
	}
	else if ( Format == ACF_IntervalFixed32NoW )
	{
		const FQuatIntervalFixed32NoW QuatIntervalFixed32NoW( Quat, Mins, Ranges );
		UnalignedWriteToStream( ByteStream, &QuatIntervalFixed32NoW, sizeof(FQuatIntervalFixed32NoW) );
	}
	else if ( Format == ACF_Float32NoW )
	{
		const FQuatFloat32NoW QuatFloat32NoW( Quat );
		UnalignedWriteToStream( ByteStream, &QuatFloat32NoW, sizeof(FQuatFloat32NoW) );
	}
}

uint8 MakeBitForFlag(uint32 Item, uint32 Position)
{
	checkSlow(Item < 2);
	return Item << Position;
}

//////////////////////////////////////////////////////////////////////////////////////
// FCompressionMemorySummary
void FCompressionMemorySummary::GatherPreCompressionStats(int32 RawSize, int32 PreCompressedSize)
{
	bUsed = true;
	TotalRaw += RawSize;
	TotalBeforeCompressed += PreCompressedSize;
	++NumberOfAnimations;
}

//////////////////////////////////////////////////////////////////////////////////////
// UAnimCompress



void UAnimCompress::PrecalculateShortestQuaternionRoutes(
	TArray<struct FRotationTrack>& RotationData)
{
	const int32 NumTracks = RotationData.Num();
	for ( int32 TrackIndex = 0 ; TrackIndex < NumTracks ; ++TrackIndex )
	{
		FRotationTrack& SrcRot	= RotationData[TrackIndex];
		for ( int32 KeyIndex = 1 ; KeyIndex < SrcRot.RotKeys.Num() ; ++KeyIndex )
		{
			const FQuat4f& R0 = SrcRot.RotKeys[KeyIndex-1];
			FQuat4f& R1 = SrcRot.RotKeys[KeyIndex];
			
			if( (R0 | R1) < 0.f )
			{
				// invert R1 so that R0|R1 will always be >=0.f
				// making the delta between them the shortest possible route
				R1 = (R1 * -1);
			}
		}
	}
}

void UAnimCompress::PadByteStream(TArray<uint8>& ByteStream, const int32 Alignment, uint8 Sentinel)
{
	int32 Pad = Align( ByteStream.Num(), Alignment ) - ByteStream.Num();
	for ( int32 i = 0 ; i < Pad ; ++i )
	{
		ByteStream.Add(Sentinel);
	}
}

namespace UE::Anim::Compression::Private
{
	static bool ShouldCompressScale(const FCompressibleAnimData& CompressibleAnimData, const TArray<FScaleTrack>& ScaleData)
	{
		if (CompressibleAnimData.bIsValidAdditive)
		{
			// We always compress scale with additive sequences
			return true;
		}

		for (int32 TrackIndex = 0; TrackIndex < ScaleData.Num(); TrackIndex++)
		{
			const FScaleTrack& ScaleTrack = ScaleData[TrackIndex];
			if (ScaleTrack.ScaleKeys.Num() != 1)
			{
				// If we have 0 keys or more than 1 key, we aren't a trivial track with a value equal to the bind pose
				// We thus need to compress scale
				return true;
			}

			const int32 BoneIndex = CompressibleAnimData.TrackToSkeletonMapTable[TrackIndex].BoneTreeIndex;
			if (!CompressibleAnimData.RefSkeleton.IsValidIndex(BoneIndex))
			{
				// This track isn't mapped to a bone, ignore it
				continue;
			}

			const FVector3f RefScale(CompressibleAnimData.RefSkeleton.GetRefBonePose()[BoneIndex].GetScale3D());
			if (!RefScale.Equals(ScaleTrack.ScaleKeys[0]))
			{
				// This trivial track isn't equal to the bind pose scale, we need to retain it
				return true;
			}
		}

		// We don't need to retain any scale, it can safely be stripped and ignored
		// At runtime, when we decompress we'll skip missing scale entries, leaving the
		// bind pose value untouched in the output buffer
		return false;
	}
}

void UAnimCompress::BitwiseCompressAnimationTracks(
	const FCompressibleAnimData& CompressibleAnimData,
	FCompressibleAnimDataResult& OutCompressedData,
	AnimationCompressionFormat TargetTranslationFormat,
	AnimationCompressionFormat TargetRotationFormat,
	AnimationCompressionFormat TargetScaleFormat,
	const TArray<FTranslationTrack>& TranslationData,
	const TArray<FRotationTrack>& RotationData,
	const TArray<FScaleTrack>& ScaleData,
	bool IncludeKeyTable)
{
	FUECompressedAnimDataMutable& AnimData = static_cast<FUECompressedAnimDataMutable&>(*OutCompressedData.AnimData);

	// Ensure supported compression formats.
	bool bInvalidCompressionFormat = false;
	if (!(TargetTranslationFormat == ACF_None) && !(TargetTranslationFormat == ACF_IntervalFixed32NoW) && !(TargetTranslationFormat == ACF_Float96NoW))
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("Engine", "UnknownTranslationCompressionFormat", "Unknown or unsupported translation compression format ({0})"), FText::AsNumber((int32)TargetTranslationFormat)));
		bInvalidCompressionFormat = true;
	}
	if (!(TargetRotationFormat >= ACF_None && TargetRotationFormat < ACF_MAX))
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("Engine", "UnknownRotationCompressionFormat", "Unknown or unsupported rotation compression format ({0})"), FText::AsNumber((int32)TargetRotationFormat)));
		bInvalidCompressionFormat = true;
	}
	if (!(TargetScaleFormat == ACF_None) && !(TargetScaleFormat == ACF_IntervalFixed32NoW) && !(TargetScaleFormat == ACF_Float96NoW))
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("Engine", "UnknownScaleCompressionFormat", "Unknown or unsupported Scale compression format ({0})"), FText::AsNumber((int32)TargetScaleFormat)));
		bInvalidCompressionFormat = true;
	}
	if (bInvalidCompressionFormat)
	{
		AnimData.TranslationCompressionFormat = ACF_None;
		AnimData.RotationCompressionFormat = ACF_None;
		AnimData.ScaleCompressionFormat = ACF_None;
		AnimData.CompressedTrackOffsets.Empty();
		AnimData.CompressedScaleOffsets.Empty();
		AnimData.CompressedByteStream.Empty();
	}
	else
	{
		AnimData.RotationCompressionFormat = TargetRotationFormat;
		AnimData.TranslationCompressionFormat = TargetTranslationFormat;
		AnimData.ScaleCompressionFormat = TargetScaleFormat;

		check(TranslationData.Num() == RotationData.Num());
		const int32 NumTracks = RotationData.Num();
		const bool bHasScale = UE::Anim::Compression::Private::ShouldCompressScale(CompressibleAnimData, ScaleData);

		if (NumTracks == 0)
		{
			UE_LOGF(LogAnimationCompression, Warning, "When compressing %ls: no key-reduced data", *CompressibleAnimData.Name);
		}

		AnimData.CompressedTrackOffsets.SetNumUninitialized(NumTracks * 4);

		// just empty it since there is chance this can be 0
		AnimData.CompressedScaleOffsets.Empty();
		// only do this if Scale exists;
		if (bHasScale)
		{
			AnimData.CompressedScaleOffsets.SetStripSize(2);
			AnimData.CompressedScaleOffsets.AddUninitialized(NumTracks);
		}

		const int32 MaxSize = CompressibleAnimData.RawAnimationData.Num() * CompressibleAnimData.NumberOfKeys * (sizeof(FVector3f) + sizeof(FQuat4f) + sizeof(FVector3f));
		AnimData.CompressedByteStream.Reset(MaxSize);

		for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
		{
			// Translation data.
			const FTranslationTrack& SrcTrans = TranslationData[TrackIndex];

			const int32 OffsetTrans = AnimData.CompressedByteStream.Num();
			const int32 NumKeysTrans = SrcTrans.PosKeys.Num();

			// Warn on empty data.
			if (NumKeysTrans == 0)
			{
				UE_LOGF(LogAnimationCompression, Warning, "When compressing %ls track %i: no translation keys", *CompressibleAnimData.Name, TrackIndex);
			}

			checkf((OffsetTrans % 4) == 0, TEXT("CompressedByteStream not aligned to four bytes"));
			AnimData.CompressedTrackOffsets[TrackIndex * 4] = OffsetTrans;
			AnimData.CompressedTrackOffsets[TrackIndex * 4 + 1] = NumKeysTrans;

			// Calculate the bounding box of the translation keys
			FBox3f PositionBounds(SrcTrans.PosKeys);

			float TransMins[3] = { (float)PositionBounds.Min.X, (float)PositionBounds.Min.Y, (float)PositionBounds.Min.Z };
			float TransRanges[3] = { float(PositionBounds.Max.X - PositionBounds.Min.X), float(PositionBounds.Max.Y - PositionBounds.Min.Y), float(PositionBounds.Max.Z - PositionBounds.Min.Z) };
			if (TransRanges[0] == 0.f) { TransRanges[0] = 1.f; }
			if (TransRanges[1] == 0.f) { TransRanges[1] = 1.f; }
			if (TransRanges[2] == 0.f) { TransRanges[2] = 1.f; }

			if (NumKeysTrans > 1)
			{
				// Write the mins and ranges if they'll be used on the other side
				if (TargetTranslationFormat == ACF_IntervalFixed32NoW)
				{
					UnalignedWriteToStream(AnimData.CompressedByteStream, TransMins, sizeof(float) * 3);
					UnalignedWriteToStream(AnimData.CompressedByteStream, TransRanges, sizeof(float) * 3);
				}

				// Pack the positions into the stream
				for (int32 KeyIndex = 0; KeyIndex < NumKeysTrans; ++KeyIndex)
				{
					const FVector3f& Vec = SrcTrans.PosKeys[KeyIndex];
					PackVectorToStream(AnimData.CompressedByteStream, TargetTranslationFormat, Vec, TransMins, TransRanges);
				}

				if (IncludeKeyTable)
				{
					// Align to four bytes.
					PadByteStream(AnimData.CompressedByteStream, 4, AnimationPadSentinel);

					// write the key table
					const int32 NumKeys = CompressibleAnimData.NumberOfKeys;
					const int32 LastFrame = NumKeys-1;
					const size_t FrameSize = NumKeys > 0xff ? sizeof(uint16) : sizeof(uint8);
					const float FrameRate = LastFrame / CompressibleAnimData.SequenceLength;

					const int32 TableSize = NumKeysTrans*FrameSize;
					const int32 TableDwords = (TableSize + 3) >> 2;
					const int32 StartingOffset = AnimData.CompressedByteStream.Num();

					for (int32 KeyIndex = 0; KeyIndex < NumKeysTrans; ++KeyIndex)
					{
						// write the frame values for each key
						float KeyTime = SrcTrans.Times[KeyIndex];
						float FrameTime = KeyTime * FrameRate;
						int32 FrameIndex = FMath::Clamp(FMath::TruncToInt(FrameTime + 0.5f), 0, LastFrame);
						UnalignedWriteToStream(AnimData.CompressedByteStream, &FrameIndex, FrameSize);
					}

					// Align to four bytes. Padding with 0's to round out the key table
					PadByteStream(AnimData.CompressedByteStream, 4, 0);

					const int32 EndingOffset = AnimData.CompressedByteStream.Num();
					check((EndingOffset - StartingOffset) == (TableDwords * 4));
				}
			}
			else if (NumKeysTrans == 1)
			{
				// A single translation key gets written out a single uncompressed float[3].
				UnalignedWriteToStream(AnimData.CompressedByteStream, &(SrcTrans.PosKeys[0]), sizeof(FVector3f));
			}
			else
			{
				UE_LOGF(LogAnimationCompression, Warning, "When compressing %ls track %i: no translation keys", *CompressibleAnimData.Name, TrackIndex);
			}

			// Align to four bytes.
			PadByteStream(AnimData.CompressedByteStream, 4, AnimationPadSentinel);

			// Compress rotation data.
			const FRotationTrack& SrcRot = RotationData[TrackIndex];
			const int32 OffsetRot = AnimData.CompressedByteStream.Num();
			const int32 NumKeysRot = SrcRot.RotKeys.Num();

			checkf((OffsetRot % 4) == 0, TEXT("CompressedByteStream not aligned to four bytes"));
			AnimData.CompressedTrackOffsets[TrackIndex * 4 + 2] = OffsetRot;
			AnimData.CompressedTrackOffsets[TrackIndex * 4 + 3] = NumKeysRot;

			if (NumKeysRot > 1)
			{
				// Calculate the min/max of the XYZ components of the quaternion
				float MinX = 1.f;
				float MinY = 1.f;
				float MinZ = 1.f;
				float MaxX = -1.f;
				float MaxY = -1.f;
				float MaxZ = -1.f;
				for (int32 KeyIndex = 0; KeyIndex < SrcRot.RotKeys.Num(); ++KeyIndex)
				{
					FQuat4f Quat(SrcRot.RotKeys[KeyIndex]);
					if (Quat.W < 0.f)
					{
						Quat.X = -Quat.X;
						Quat.Y = -Quat.Y;
						Quat.Z = -Quat.Z;
						Quat.W = -Quat.W;
					}
					Quat.Normalize();

					MinX = FMath::Min(MinX, Quat.X);
					MaxX = FMath::Max(MaxX, Quat.X);
					MinY = FMath::Min(MinY, Quat.Y);
					MaxY = FMath::Max(MaxY, Quat.Y);
					MinZ = FMath::Min(MinZ, Quat.Z);
					MaxZ = FMath::Max(MaxZ, Quat.Z);
				}
				const float Mins[3] = { MinX,		MinY,		MinZ };
				float Ranges[3] = { MaxX - MinX,	MaxY - MinY,	MaxZ - MinZ };
				if (Ranges[0] == 0.f) { Ranges[0] = 1.f; }
				if (Ranges[1] == 0.f) { Ranges[1] = 1.f; }
				if (Ranges[2] == 0.f) { Ranges[2] = 1.f; }

				// Write the mins and ranges if they'll be used on the other side
				if (TargetRotationFormat == ACF_IntervalFixed32NoW)
				{
					UnalignedWriteToStream(AnimData.CompressedByteStream, Mins, sizeof(float) * 3);
					UnalignedWriteToStream(AnimData.CompressedByteStream, Ranges, sizeof(float) * 3);
				}

				// n elements of the compressed type.
				for (int32 KeyIndex = 0; KeyIndex < SrcRot.RotKeys.Num(); ++KeyIndex)
				{
					const FQuat4f& Quat = SrcRot.RotKeys[KeyIndex];
					PackQuaternionToStream(AnimData.CompressedByteStream, TargetRotationFormat, Quat, Mins, Ranges);
				}

				// n elements of frame indices
				if (IncludeKeyTable)
				{
					// Align to four bytes.
					PadByteStream(AnimData.CompressedByteStream, 4, AnimationPadSentinel);

					// write the key table
					const int32 NumKeys = CompressibleAnimData.NumberOfKeys;
					const int32 LastFrame= NumKeys-1;
					const size_t FrameSize= NumKeys > 0xff ? sizeof(uint16) : sizeof(uint8);
					const float FrameRate = LastFrame / CompressibleAnimData.SequenceLength;

					const int32 TableSize = NumKeysRot*FrameSize;
					const int32 TableDwords = (TableSize + 3) >> 2;
					const int32 StartingOffset = AnimData.CompressedByteStream.Num();

					for (int32 KeyIndex = 0; KeyIndex < NumKeysRot; ++KeyIndex)
					{
						// write the frame values for each key
						float KeyTime = SrcRot.Times[KeyIndex];
						float FrameTime = KeyTime * FrameRate;
						int32 FrameIndex = FMath::Clamp(FMath::TruncToInt(FrameTime + 0.5f), 0, LastFrame);
						UnalignedWriteToStream(AnimData.CompressedByteStream, &FrameIndex, FrameSize);
					}

					// Align to four bytes. Padding with 0's to round out the key table
					PadByteStream(AnimData.CompressedByteStream, 4, 0);

					const int32 EndingOffset = AnimData.CompressedByteStream.Num();
					check((EndingOffset - StartingOffset) == (TableDwords * 4));

				}
			}
			else if (NumKeysRot == 1)
			{
				// For a rotation track of n=1 keys, the single key is packed as an FQuatFloat96NoW.
				const FQuat4f& Quat = SrcRot.RotKeys[0];
				const FQuatFloat96NoW QuatFloat96NoW(Quat);
				UnalignedWriteToStream(AnimData.CompressedByteStream, &QuatFloat96NoW, sizeof(FQuatFloat96NoW));
			}
			else
			{
				UE_LOGF(LogAnimationCompression, Warning, "When compressing %ls track %i: no rotation keys", *CompressibleAnimData.Name, TrackIndex);
			}


			// Align to four bytes.
			PadByteStream(AnimData.CompressedByteStream, 4, AnimationPadSentinel);

			// we also should do this only when scale exists. 
			if (bHasScale)
			{
				const FScaleTrack& SrcScale = ScaleData[TrackIndex];

				const int32 OffsetScale = AnimData.CompressedByteStream.Num();
				const int32 NumKeysScale = SrcScale.ScaleKeys.Num();

				checkf((OffsetScale % 4) == 0, TEXT("CompressedByteStream not aligned to four bytes"));
				AnimData.CompressedScaleOffsets.SetOffsetData(TrackIndex, 0, OffsetScale);
				AnimData.CompressedScaleOffsets.SetOffsetData(TrackIndex, 1, NumKeysScale);

				// Calculate the bounding box of the Scalelation keys
				FBox3f ScaleBoundsBounds(SrcScale.ScaleKeys);

				float ScaleMins[3] = { (float)ScaleBoundsBounds.Min.X, (float)ScaleBoundsBounds.Min.Y, (float)ScaleBoundsBounds.Min.Z };
				float ScaleRanges[3] = { float(ScaleBoundsBounds.Max.X - ScaleBoundsBounds.Min.X), float(ScaleBoundsBounds.Max.Y - ScaleBoundsBounds.Min.Y), float(ScaleBoundsBounds.Max.Z - ScaleBoundsBounds.Min.Z) };
				// @todo - this isn't good for scale 
				// 			if ( ScaleRanges[0] == 0.f ) { ScaleRanges[0] = 1.f; }
				// 			if ( ScaleRanges[1] == 0.f ) { ScaleRanges[1] = 1.f; }
				// 			if ( ScaleRanges[2] == 0.f ) { ScaleRanges[2] = 1.f; }

				if (NumKeysScale > 1)
				{
					// Write the mins and ranges if they'll be used on the other side
					if (TargetScaleFormat == ACF_IntervalFixed32NoW)
					{
						UnalignedWriteToStream(AnimData.CompressedByteStream, ScaleMins, sizeof(float) * 3);
						UnalignedWriteToStream(AnimData.CompressedByteStream, ScaleRanges, sizeof(float) * 3);
					}

					// Pack the positions into the stream
					for (int32 KeyIndex = 0; KeyIndex < NumKeysScale; ++KeyIndex)
					{
						const FVector3f& Vec = SrcScale.ScaleKeys[KeyIndex];
						PackVectorToStream(AnimData.CompressedByteStream, TargetScaleFormat, Vec, ScaleMins, ScaleRanges);
					}

					if (IncludeKeyTable)
					{
						// Align to four bytes.
						PadByteStream(AnimData.CompressedByteStream, 4, AnimationPadSentinel);

						// write the key table
						const int32 NumKeys = CompressibleAnimData.NumberOfKeys;
						const int32 LastFrame = NumKeys-1;
						const size_t FrameSize = NumKeys > 0xff ? sizeof(uint16) : sizeof(uint8);
						const float FrameRate = LastFrame / CompressibleAnimData.SequenceLength;

						const int32 TableSize = NumKeysScale*FrameSize;
						const int32 TableDwords = (TableSize + 3) >> 2;
						const int32 StartingOffset = AnimData.CompressedByteStream.Num();

						for (int32 KeyIndex = 0; KeyIndex < NumKeysScale; ++KeyIndex)
						{
							// write the frame values for each key
							float KeyTime = SrcScale.Times[KeyIndex];
							float FrameTime = KeyTime * FrameRate;
							int32 FrameIndex = FMath::Clamp(FMath::TruncToInt(FrameTime + 0.5f), 0, LastFrame);
							UnalignedWriteToStream(AnimData.CompressedByteStream, &FrameIndex, FrameSize);
						}

						// Align to four bytes. Padding with 0's to round out the key table
						PadByteStream(AnimData.CompressedByteStream, 4, 0);

						const int32 EndingOffset = AnimData.CompressedByteStream.Num();
						check((EndingOffset - StartingOffset) == (TableDwords * 4));
					}
				}
				else if (NumKeysScale == 1)
				{
					// A single Scalelation key gets written out a single uncompressed float[3].
					UnalignedWriteToStream(AnimData.CompressedByteStream, &(SrcScale.ScaleKeys[0]), sizeof(FVector3f));
				}
				else
				{
					UE_LOGF(LogAnimationCompression, Warning, "When compressing %ls track %i: no Scale keys", *CompressibleAnimData.Name, TrackIndex);
				}

				// Align to four bytes.
				PadByteStream(AnimData.CompressedByteStream, 4, AnimationPadSentinel);
			}
		}

		// Trim unused memory.
		AnimData.CompressedByteStream.Shrink();
	}
}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
bool UAnimCompress::Compress(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult)
{
	FUECompressedAnimDataMutable AnimDataMutable;
	AnimDataMutable.CompressedNumberOfKeys = CompressibleAnimData.NumberOfKeys;

	OutResult.Codec = this;
	OutResult.AnimData = TUniquePtr<ICompressedAnimData>(&AnimDataMutable);

	const bool bSuccess = DoReduction(CompressibleAnimData, OutResult);

	// Clear without free since we were on the stack
	(void)OutResult.AnimData.Release();

	if (bSuccess)
	{
		// Finished compressing, coalesce the buffers
		AnimDataMutable.BuildFinalBuffer(OutResult.CompressedByteStream);

		// Build our read-only version from the mutable source
		TUniquePtr<FUECompressedAnimData> AnimData = MakeUnique<FUECompressedAnimData>(AnimDataMutable);

		// The buffers will point to the mutable data, bind to the byte stream instead
		AnimData->InitViewsFromBuffer(OutResult.CompressedByteStream);

		OutResult.AnimData = MoveTemp(AnimData);
	}

	return bSuccess;
}
#endif

TUniquePtr<ICompressedAnimData> UAnimCompress::AllocateAnimData() const
{
	return MakeUnique<FUECompressedAnimData>();
}

void UAnimCompress::ByteSwapIn(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryReader& MemoryStream) const
{
	static_cast<FUECompressedAnimData&>(AnimData).ByteSwapIn(CompressedData, MemoryStream);
}

void UAnimCompress::ByteSwapOut(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryWriter& MemoryStream) const
{
	static_cast<FUECompressedAnimData&>(AnimData).ByteSwapOut(CompressedData, MemoryStream);
}

void UAnimCompress::DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const
{
	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

	AnimData.TranslationCodec->GetPoseTranslations(OutAtoms, TranslationPairs, DecompContext);

	AnimData.RotationCodec->GetPoseRotations(OutAtoms, RotationPairs, DecompContext);

	// Scale can be stripped when it is trivial and equal to the bind pose
	// When this is the case, we assume that the output pose has already been populated with the bind pose
	if (AnimData.CompressedScaleOffsets.IsValid())
	{
		AnimData.ScaleCodec->GetPoseScales(OutAtoms, ScalePairs, DecompContext);
	}
}

void UAnimCompress::DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const
{
	// Initialize to identity to set the scale and in case of a missing rotation or translation codec
	OutAtom.SetIdentity();

	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

	// decompress the translation component using the proper method
	((AnimEncodingLegacyBase*)AnimData.TranslationCodec)->GetBoneAtomTranslation(OutAtom, DecompContext, TrackIndex);

	// decompress the rotation component using the proper method
	((AnimEncodingLegacyBase*)AnimData.RotationCodec)->GetBoneAtomRotation(OutAtom, DecompContext, TrackIndex);

	// we assume scale keys can be empty, so only extract if we have valid keys
	if (AnimData.CompressedScaleOffsets.IsValid())
	{
		// decompress the rotation component using the proper method
		((AnimEncodingLegacyBase*)AnimData.ScaleCodec)->GetBoneAtomScale(OutAtom, DecompContext, TrackIndex);
	}
	else
	{
		// If scale is stripped, we must output the bind pose value since it's been stripped
		if (DecompContext.IsAdditiveAnimation())
		{
			// Additive animations use the additive identity
			OutAtom.SetScale3D(FVector::ZeroVector);
		}
		else
		{
			checkf(DecompContext.GetRefLocalPoses().Num() > 0, TEXT("Reference pose must be provided in the FAnimSequenceDecompressionContext constructor"));
			checkf(DecompContext.GetTrackToSkeletonMap().Num() > 0, TEXT("TrackToSkeletonMap must be provided in the FAnimSequenceDecompressionContext constructor"));

			const int32 BoneIndex = DecompContext.GetTrackToSkeletonMap()[TrackIndex].BoneTreeIndex;
			const FTransform& BindTransform = DecompContext.GetRefLocalPoses()[BoneIndex];

			OutAtom.SetScale3D(BindTransform.GetScale3D());
		}
	}
}

#if WITH_EDITOR

void UAnimCompress::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);

	uint8 TCF, RCF, SCF;
	TCF = (uint8)TranslationCompressionFormat.GetValue();
	RCF = (uint8)RotationCompressionFormat.GetValue();
	SCF = (uint8)ScaleCompressionFormat.GetValue();

	Ar << TCF << RCF << SCF;

	// Additive sequences use the additive identity as their bind pose, scale is never stripped
	if (!KeyArgs.AnimSequence.IsValidAdditive())
	{
		// We have to include the bind pose in the DDC key.
		// If a sequence is compressed with bind pose A, and we strip a few bones and later modify the bind pose,
		// bind pose B might now contain values that would not be stripped in our sequence.
		// To avoid data being stale, the DDC must reflect this.

		const USkeleton* Skeleton = KeyArgs.AnimSequence.GetSkeleton();
		const TArray<FTransform>& BindPose = Skeleton->GetRefLocalPoses();
		for (const FTransform& BoneBindTransform : BindPose)
		{
			// We only strip scale

			FVector Scale = BoneBindTransform.GetScale3D();
			Ar << Scale;
		}
	}
}

void UAnimCompress::FilterTrivialPositionKeys(
	FTranslationTrack& Track, 
	float MaxPosDelta)
{
	const int32 KeyCount = Track.Times.Num();
	check( Track.PosKeys.Num() == Track.Times.Num() );

	// Only bother doing anything if we have some keys!
	if( KeyCount > 1 )
	{
		const FVector3f& FirstPos = Track.PosKeys[0];

		bool bFramesIdentical = true;
		for(int32 KeyIndex=1; KeyIndex < KeyCount; ++KeyIndex)
		{
			const FVector3f& ThisPos = Track.PosKeys[KeyIndex];

			if( FMath::Abs(ThisPos.X - FirstPos.X) > MaxPosDelta || 
				FMath::Abs(ThisPos.Y - FirstPos.Y) > MaxPosDelta || 
				FMath::Abs(ThisPos.Z - FirstPos.Z) > MaxPosDelta )
			{
				bFramesIdentical = false;
				break;
			}
		}

		// If all keys are the same, remove all but first frame
		if( bFramesIdentical )
		{
			Track.PosKeys.RemoveAt(1, Track.PosKeys.Num()- 1);
			Track.PosKeys.Shrink();
			Track.Times.RemoveAt(1, Track.Times.Num()- 1);
			Track.Times.Shrink();
			Track.Times[0] = 0.0f;
		}
	}
}

void UAnimCompress::FilterTrivialPositionKeys(
	TArray<FTranslationTrack>& InputTracks, 
	float MaxPosDelta)
{
	const int32 NumTracks = InputTracks.Num();
	for( int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex )
	{
		FTranslationTrack& Track = InputTracks[TrackIndex];
		FilterTrivialPositionKeys(Track, MaxPosDelta);
	}
}


void UAnimCompress::FilterTrivialScaleKeys(
	FScaleTrack& Track, 
	float MaxScaleDelta)
{
	const int32 KeyCount = Track.Times.Num();
	check( Track.ScaleKeys.Num() == Track.Times.Num() );

	// Only bother doing anything if we have some keys!
	if( KeyCount > 1 )
	{
		const FVector3f& FirstPos = Track.ScaleKeys[0];

		bool bFramesIdentical = true;
		for(int32 KeyIndex=1; KeyIndex < KeyCount; ++KeyIndex)
		{
			const FVector3f& ThisPos = Track.ScaleKeys[KeyIndex];

			if( FMath::Abs(ThisPos.X - FirstPos.X) > MaxScaleDelta || 
				FMath::Abs(ThisPos.Y - FirstPos.Y) > MaxScaleDelta || 
				FMath::Abs(ThisPos.Z - FirstPos.Z) > MaxScaleDelta )
			{
				bFramesIdentical = false;
				break;
			}
		}

		// If all keys are the same, remove all but first frame
		if( bFramesIdentical )
		{
			Track.ScaleKeys.RemoveAt(1, Track.ScaleKeys.Num()- 1);
			Track.ScaleKeys.Shrink();
			Track.Times.RemoveAt(1, Track.Times.Num()- 1);
			Track.Times.Shrink();
			Track.Times[0] = 0.0f;
		}
	}
}

void UAnimCompress::FilterTrivialScaleKeys(
	TArray<FScaleTrack>& InputTracks, 
	float MaxScaleDelta)
{
	const int32 NumTracks = InputTracks.Num();
	for( int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex )
	{
		FScaleTrack& Track = InputTracks[TrackIndex];
		FilterTrivialScaleKeys(Track, MaxScaleDelta);
	}
}

void UAnimCompress::FilterTrivialRotationKeys(
	FRotationTrack& Track, 
	float MaxRotDelta)
{
	const int32 KeyCount = Track.Times.Num();
	check( Track.RotKeys.Num() == Track.Times.Num() );

	// Only bother doing anything if we have some keys!
	if(KeyCount > 1)
	{
		const FQuat4f& FirstRot = Track.RotKeys[0];
		bool bFramesIdentical = true;
		for(int32 KeyIndex=1; KeyIndex<KeyCount; ++KeyIndex)
		{
			if( FQuat4f::Error(FirstRot, Track.RotKeys[KeyIndex]) > MaxRotDelta )
			{
				bFramesIdentical = false;
				break;
			}
		}

		if(bFramesIdentical)
		{
			Track.RotKeys.RemoveAt(1, Track.RotKeys.Num()- 1);
			Track.RotKeys.Shrink();
			Track.Times.RemoveAt(1, Track.Times.Num()- 1);
			Track.Times.Shrink();
			Track.Times[0] = 0.0f;
		}			
	}
}


void UAnimCompress::FilterTrivialRotationKeys(
	TArray<FRotationTrack>& InputTracks, 
	float MaxRotDelta)
{
	const int32 NumTracks = InputTracks.Num();
	for( int32 TrackIndex = 0 ; TrackIndex < NumTracks ; ++TrackIndex )
	{
		FRotationTrack& Track = InputTracks[TrackIndex];
		FilterTrivialRotationKeys(Track, MaxRotDelta);
	}
}


void UAnimCompress::FilterTrivialKeys(
	TArray<FTranslationTrack>& PositionTracks,
	TArray<FRotationTrack>& RotationTracks, 
	TArray<FScaleTrack>& ScaleTracks,
	float MaxPosDelta,
	float MaxRotDelta,
	float MaxScaleDelta)
{
	FilterTrivialRotationKeys(RotationTracks, MaxRotDelta);
	FilterTrivialPositionKeys(PositionTracks, MaxPosDelta);
	FilterTrivialScaleKeys(ScaleTracks, MaxScaleDelta);
}


void UAnimCompress::FilterIntermittentPositionKeys(
	FTranslationTrack& Track, 
	int32 StartIndex,
	int32 Interval)
{
	const int32 KeyCount = Track.Times.Num();
	const int32 FinalIndex = KeyCount - 1;
	StartIndex = FMath::Min(StartIndex, FinalIndex);

	check(Track.Times.Num() == Track.PosKeys.Num());

	TArray<FVector3f> NewPosKeys;
	TArray<float> NewTimes;

	NewTimes.Empty(KeyCount);
	NewPosKeys.Empty(KeyCount);

	// step through and retain the desired interval
	for (int32 KeyIndex = StartIndex; KeyIndex < KeyCount; KeyIndex += Interval )
	{
		NewTimes.Add( Track.Times[KeyIndex] );
		NewPosKeys.Add( Track.PosKeys[KeyIndex] );
	}

	NewTimes.Shrink();
	NewPosKeys.Shrink();

	Track.Times = NewTimes;
	Track.PosKeys = NewPosKeys;
}


void UAnimCompress::FilterIntermittentPositionKeys(
	TArray<FTranslationTrack>& PositionTracks, 
	int32 StartIndex,
	int32 Interval)
{
	const int32 NumPosTracks = PositionTracks.Num();

	// copy intermittent position keys
	for( int32 TrackIndex = 0; TrackIndex < NumPosTracks; ++TrackIndex )
	{
		FTranslationTrack& OldTrack	= PositionTracks[TrackIndex];
		FilterIntermittentPositionKeys(OldTrack, StartIndex, Interval);
	}
}


void UAnimCompress::FilterIntermittentRotationKeys(
	FRotationTrack& Track,
	int32 StartIndex,
	int32 Interval)
{
	const int32 KeyCount = Track.Times.Num();
	const int32 FinalIndex = KeyCount-1;
	StartIndex = FMath::Min(StartIndex, FinalIndex);

	check(Track.Times.Num() == Track.RotKeys.Num());

	TArray<FQuat4f> NewRotKeys;
	TArray<float> NewTimes;

	NewTimes.Empty(KeyCount);
	NewRotKeys.Empty(KeyCount);

	// step through and retain the desired interval
	for (int32 KeyIndex = StartIndex; KeyIndex < KeyCount; KeyIndex += Interval )
	{
		NewTimes.Add( Track.Times[KeyIndex] );
		NewRotKeys.Add( Track.RotKeys[KeyIndex] );
	}

	NewTimes.Shrink();
	NewRotKeys.Shrink();
	Track.Times = NewTimes;
	Track.RotKeys = NewRotKeys;
}


void UAnimCompress::FilterIntermittentRotationKeys(
	TArray<FRotationTrack>& RotationTracks, 
	int32 StartIndex,
	int32 Interval)
{
	const int32 NumRotTracks = RotationTracks.Num();

	// copy intermittent position keys
	for( int32 TrackIndex = 0; TrackIndex < NumRotTracks; ++TrackIndex )
	{
		FRotationTrack& OldTrack = RotationTracks[TrackIndex];
		FilterIntermittentRotationKeys(OldTrack, StartIndex, Interval);
	}
}


void UAnimCompress::FilterIntermittentKeys(
	TArray<FTranslationTrack>& PositionTracks, 
	TArray<FRotationTrack>& RotationTracks, 
	int32 StartIndex,
	int32 Interval)
{
	FilterIntermittentPositionKeys(PositionTracks, StartIndex, Interval);
	FilterIntermittentRotationKeys(RotationTracks, StartIndex, Interval);
}


void UAnimCompress::SeparateRawDataIntoTracks(
	const TArray<FRawAnimSequenceTrack>& RawAnimData,
	float SequenceLength,
	TArray<FTranslationTrack>& OutTranslationData,
	TArray<FRotationTrack>& OutRotationData, 
	TArray<FScaleTrack>& OutScaleData)
{
	const int32 NumTracks = RawAnimData.Num();

	OutTranslationData.Reset( NumTracks );
	OutRotationData.Reset( NumTracks );
	OutScaleData.Reset( NumTracks );
	OutTranslationData.SetNumZeroed( NumTracks );
	OutRotationData.SetNumZeroed( NumTracks );
	OutScaleData.SetNumZeroed( NumTracks );

	// only compress scale if it has valid scale keys
	bool bCompressScaleKeys = false;

	for ( int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex )
	{
		const FRawAnimSequenceTrack& RawTrack	= RawAnimData[TrackIndex];
		FTranslationTrack&	TranslationTrack	= OutTranslationData[TrackIndex];
		FRotationTrack&		RotationTrack		= OutRotationData[TrackIndex];
		FScaleTrack&		ScaleTrack			= OutScaleData[TrackIndex];

		const int32 PrevNumPosKeys = RawTrack.PosKeys.Num();
		const int32 PrevNumRotKeys = RawTrack.RotKeys.Num();
		const bool	bHasScale = (RawTrack.ScaleKeys.Num() != 0);
		bCompressScaleKeys |= bHasScale;

		// Do nothing if the data for this track is empty.
		if( PrevNumPosKeys == 0 || PrevNumRotKeys == 0 )
		{
			continue;
		}

		// Copy over position keys.
		TranslationTrack.PosKeys = RawTrack.PosKeys;

		// Copy over rotation keys.
		RotationTrack.RotKeys = RawTrack.RotKeys;

		// Set times for the translation track.
		const int32 NumPosKeys = TranslationTrack.PosKeys.Num();
		TranslationTrack.Times.SetNumUninitialized(NumPosKeys);
		if ( NumPosKeys > 1 )
		{
			const float PosFrameInterval = SequenceLength / static_cast<float>(NumPosKeys-1);
			for ( int32 PosIndex = 0; PosIndex < NumPosKeys; ++PosIndex )
			{
				TranslationTrack.Times[PosIndex] = PosIndex * PosFrameInterval;
			}
		}
		else
		{
			TranslationTrack.Times[0] = 0.f;
		}

		// Set times for the rotation track.
		const int32 NumRotKeys = RotationTrack.RotKeys.Num();
		RotationTrack.Times.SetNumUninitialized(NumRotKeys);
		if ( NumRotKeys > 1 )
		{
			// If # of keys match between translation and rotation, re-use the timing values
			if (NumRotKeys == NumPosKeys)
			{
				RotationTrack.Times = TranslationTrack.Times;
			}
			else
			{
				const float RotFrameInterval = SequenceLength / static_cast<float>(NumRotKeys-1);
				for ( int32 RotIndex = 0; RotIndex < NumRotKeys; ++RotIndex )
				{
					RotationTrack.Times[RotIndex] = RotIndex * RotFrameInterval;
				}
			}
		
		}
		else
		{
			RotationTrack.Times[0] = 0.f;
		}

		if (bHasScale)
		{
			// Copy over scalekeys.
			ScaleTrack.ScaleKeys = RawTrack.ScaleKeys;
			// Set times for the rotation track.
			const int32 NumScaleKeys = ScaleTrack.ScaleKeys.Num();
			ScaleTrack.Times.SetNumUninitialized(NumScaleKeys);
			if ( NumScaleKeys > 1 )
			{
				// If # of keys match between translation and scale, re-use the timing values
				if (NumScaleKeys == NumPosKeys)
				{
					ScaleTrack.Times = TranslationTrack.Times;
				}
				else
				{
					const float ScaleFrameInterval = SequenceLength / static_cast<float>(NumScaleKeys-1);
					for ( int32 ScaleIndex = 0; ScaleIndex < NumScaleKeys; ++ScaleIndex )
					{
						ScaleTrack.Times[ScaleIndex] = ScaleIndex * ScaleFrameInterval;
					}
				}
			}
			else
			{
				ScaleTrack.Times[0] = 0.f;
			}
		}

		// Trim unused memory.
		TranslationTrack.PosKeys.Shrink();
		TranslationTrack.Times.Shrink();
		RotationTrack.RotKeys.Shrink();
		RotationTrack.Times.Shrink();
		ScaleTrack.ScaleKeys.Shrink();
		ScaleTrack.Times.Shrink();
	}

	// if nothing to compress, empty the ScaleData
	// that way we don't have to worry about compressing scale data. 
	if (!bCompressScaleKeys)
	{
		OutScaleData.Empty();
	}
}
#endif