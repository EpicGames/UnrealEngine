// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompress_BitwiseCompressionOnly.cpp: Bitwise animation compression only; performs no key reduction.
=============================================================================*/ 

#include "Animation/AnimCompress_BitwiseCompressOnly.h"
#include "Animation/AnimSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCompress_BitwiseCompressOnly)

UAnimCompress_BitwiseCompressOnly::UAnimCompress_BitwiseCompressOnly(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = TEXT("Bitwise Compress Only");
}

#if WITH_EDITOR
bool UAnimCompress_BitwiseCompressOnly::DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult)
{
#if WITH_EDITORONLY_DATA
	// split the raw data into tracks
	TArray<FTranslationTrack> TranslationData;
	TArray<FRotationTrack> RotationData;
	TArray<FScaleTrack> ScaleData;
	SeparateRawDataIntoTracks( CompressibleAnimData.RawAnimationData, CompressibleAnimData.SequenceLength, TranslationData, RotationData, ScaleData );

	// remove obviously redundant keys from the source data
	FilterTrivialKeys(TranslationData, RotationData, ScaleData, TRANSLATION_ZEROING_THRESHOLD, QUATERNION_ZEROING_THRESHOLD, SCALE_ZEROING_THRESHOLD);

	// record the proper runtime decompressor to use
	FUECompressedAnimDataMutable& AnimData = static_cast<FUECompressedAnimDataMutable&>(*OutResult.AnimData);
	AnimData.KeyEncodingFormat = AKF_ConstantKeyLerp;
	AnimData.RotationCompressionFormat = RotationCompressionFormat;
	AnimData.TranslationCompressionFormat = TranslationCompressionFormat;
	AnimData.ScaleCompressionFormat = ScaleCompressionFormat;
	AnimationFormat_SetInterfaceLinks(AnimData);

	// bitwise compress the tracks into the anim sequence buffers
	BitwiseCompressAnimationTracks(
		CompressibleAnimData,
		OutResult,
		static_cast<AnimationCompressionFormat>(TranslationCompressionFormat),
		static_cast<AnimationCompressionFormat>(RotationCompressionFormat),
		static_cast<AnimationCompressionFormat>(ScaleCompressionFormat),
		TranslationData,
		RotationData,
		ScaleData);

	// We could be invalid, set the links again
	AnimationFormat_SetInterfaceLinks(AnimData);
#endif // WITH_EDITORONLY_DATA
	return true;
}

#endif // WITH_EDITOR

