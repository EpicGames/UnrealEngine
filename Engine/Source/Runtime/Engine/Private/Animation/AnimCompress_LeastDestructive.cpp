// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompress_LeastDestructive.cpp: Uses the Bitwise compressor with really light settings
=============================================================================*/ 

#include "Animation/AnimCompress_LeastDestructive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCompress_LeastDestructive)

UAnimCompress_LeastDestructive::UAnimCompress_LeastDestructive(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = TEXT("Least Destructive");
	TranslationCompressionFormat = ACF_None;
	RotationCompressionFormat = ACF_Float96NoW;
}

