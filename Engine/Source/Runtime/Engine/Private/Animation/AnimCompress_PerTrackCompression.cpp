// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompress_PerTrackCompression.cpp
=============================================================================*/ 

#include "Animation/AnimCompress_PerTrackCompression.h"
#include "AnimEncoding_PerTrackCompression.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "AnimationUtils.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCompress_PerTrackCompression)

struct FAnimSetMeshLinkup;

struct FPerTrackCachedInfo
{
	/** Used as a sanity check to validate the cache */
	const FAnimSetMeshLinkup* AnimLinkup;

	/** Contains the maximum end effector errors from probe perturbations throughout the skeleton */
	TArray<FAnimPerturbationError> PerTrackErrors;

	/** Contains the height of each track within the skeleton */
	TArray<int32> TrackHeights;
};


/**
 * Structure that carries compression settings used in FPerTrackCompressor
 */
struct FPerTrackParams
{
	float MaxZeroingThreshold;

	int32 NumberOfKeys;

	float SequenceLength;

	bool bIncludeKeyTable;
};

/**
 * This class compresses a single rotation or translation track into an internal buffer, keeping error metrics as it goes.
 */
class FPerTrackCompressor
{
public:
	// Used during compression
	float MaxError;
	double SumError;

	// Results of compression
	TArray<uint8> CompressedBytes;
	AnimationCompressionFormat ActualCompressionMode;
	int32 ActualKeyFlags;

	/** Does the compression scheme need a key->frame table (needed if the keys are spaced non-uniformly in time) */
	bool bReallyNeedsFrameTable;

protected:
	/** Resets the compression buffer to defaults (no data) */
	void Reset()
	{
		MaxError = 0.0f;
		SumError = 0.0;
		bReallyNeedsFrameTable = false;
		ActualCompressionMode = ACF_None;
		ActualKeyFlags = 0;
		CompressedBytes.Reset();
	}

	/**
	 * Creates a header integer with four fields:
	 *   NumKeys can be no more than 24 bits (positions 0..23)
	 *   KeyFlags can be no more than 3 bits (positions 24..27)
	 *   bReallyNeedsFrameTable is a single bit (position 27)
	 *   KeyFormat can be no more than 4 bits (positions 31..28)
	 *
	 *   Also updates the ActualCompressionMode field
	 */
	int32 MakeHeader(const int32 NumKeys, const AnimationCompressionFormat KeyFormat, const int32 KeyFlags)
	{
		ActualCompressionMode = KeyFormat;
		ActualKeyFlags = KeyFlags;
		return FAnimationCompression_PerTrackUtils::MakeHeader(NumKeys, (int32)KeyFormat, KeyFlags, bReallyNeedsFrameTable);
	}

	/** Ensures that the CompressedBytes output stream is a multiple of 4 bytes long */
	void PadOutputStream()
	{
		const uint8 PadSentinel = 85; //(1<<1)+(1<<3)+(1<<5)+(1<<7)

		const int32 PadLength = Align(CompressedBytes.Num(), 4) - CompressedBytes.Num();
		for (int32 i = 0; i < PadLength; ++i)
		{
			CompressedBytes.Add(PadSentinel);
		}
	}

	/** Writes Length bytes from Data to the output stream */
	void AppendBytes(const void* Data, int32 Length)
	{
		const int32 Offset = CompressedBytes.AddUninitialized(Length);
		FMemory::Memcpy(CompressedBytes.GetData() + Offset, Data, Length);
	}

	void CompressTranslation_Identity(const FTranslationTrack& TranslationData)
	{
		// Compute the error when using this compression type (how far off from (0,0,0) are they?)
		const int32 NumKeys = TranslationData.PosKeys.Num();
		for (int32 i = 0; i < NumKeys; ++i)
		{
			float Error = TranslationData.PosKeys[i].Size();
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
		ActualCompressionMode = ACF_Identity;

		// Add nothing to compressed bytes; this type gets flagged extra-special, back at the offset table
	}

	void CompressTranslation_16_16_16(const FTranslationTrack& TranslationData, float ZeroingThreshold)
	{
		const int32 NumKeys = TranslationData.PosKeys.Num();

		// Determine the bounds
		const FBox3f KeyBounds(TranslationData.PosKeys.GetData(), NumKeys);
		const bool bHasX = (FMath::Abs(KeyBounds.Max.X) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.X) >= ZeroingThreshold);
		const bool bHasY = (FMath::Abs(KeyBounds.Max.Y) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Y) >= ZeroingThreshold);
		const bool bHasZ = (FMath::Abs(KeyBounds.Max.Z) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Z) >= ZeroingThreshold);

		if (!bHasX && !bHasY && !bHasZ)
		{
			// No point in using this over the identity encoding
			CompressTranslation_Identity(TranslationData);
			return;
		}

		// Write the header out
		const int32 Header = MakeHeader(NumKeys, ACF_Fixed48NoW, (bHasX ? 1 : 0) | ((bHasY ? 1 : 0)<<1) | ((bHasZ ? 1 : 0)<<2));
		AppendBytes(&Header, sizeof(Header));

		// Write the keys for the non-zero components
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FVector3f& V = TranslationData.PosKeys[i];
			
			uint16 X = 0;
			uint16 Y = 0;
			uint16 Z = 0;

			if (bHasX)
			{
				X = FAnimationCompression_PerTrackUtils::CompressFixed16(V.X, AnimationCompressionUtils::LogScale);
				AppendBytes(&X, sizeof(X));
			}
			if (bHasY)
			{
				Y = FAnimationCompression_PerTrackUtils::CompressFixed16(V.Y, AnimationCompressionUtils::LogScale);
				AppendBytes(&Y, sizeof(Y));
			}
			if (bHasZ)
			{
				Z = FAnimationCompression_PerTrackUtils::CompressFixed16(V.Z, AnimationCompressionUtils::LogScale);
				AppendBytes(&Z, sizeof(Z));
			}

			const FVector3f DecompressedV(
				bHasX ? FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(X) : 0.0f,
				bHasY ? FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(Y) : 0.0f,
				bHasZ ? FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(Z) : 0.0f);

			const float Error = (V - DecompressedV).Size();
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
	}

	void CompressTranslation_Uncompressed(const FTranslationTrack& TranslationData, float ZeroingThreshold)
	{
		const int32 NumKeys = TranslationData.PosKeys.Num();

		// Determine the bounds
		const FBox3f KeyBounds(TranslationData.PosKeys.GetData(), NumKeys);
		const bool bHasX = (FMath::Abs(KeyBounds.Max.X) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.X) >= ZeroingThreshold);
		const bool bHasY = (FMath::Abs(KeyBounds.Max.Y) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Y) >= ZeroingThreshold);
		const bool bHasZ = (FMath::Abs(KeyBounds.Max.Z) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Z) >= ZeroingThreshold);

		if( !bHasX && !bHasY && !bHasZ )
		{
			// No point in using this over the identity encoding
			CompressTranslation_Identity(TranslationData);
			return;
		}

		// Write the header out
		const int32 Header = MakeHeader(NumKeys, ACF_Float96NoW, (bHasX ? 1 : 0) | ((bHasY ? 1 : 0)<<1) | ((bHasZ ? 1 : 0)<<2));
		AppendBytes(&Header, sizeof(Header));

		// Write the keys out
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FVector3f& V = TranslationData.PosKeys[i];
			if( bHasX )
			{
				AppendBytes(&(V.X), sizeof(float));
			}
			if( bHasY )
			{
				AppendBytes(&(V.Y), sizeof(float));
			}
			if( bHasZ )
			{
				AppendBytes(&(V.Z), sizeof(float));
			}
		}

		// No error, it's a perfect encoding
		MaxError = 0.0f;
		SumError = 0.0;
	}

	// Encode a 0..1 interval in 10:11:11 (X and Z swizzled in the 11:11:10 source because Z is more important in most animations)
	// and store an uncompressed bounding box at the start of the track to scale that 0..1 back up
	void CompressTranslation_10_11_11(const FTranslationTrack& TranslationData, float ZeroingThreshold)
	{
		const int32 NumKeys = TranslationData.PosKeys.Num();

		// Determine the bounds
		const FBox3f KeyBounds(TranslationData.PosKeys.GetData(), NumKeys);
		const bool bHasX = (FMath::Abs(KeyBounds.Max.X) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.X) >= ZeroingThreshold);
		const bool bHasY = (FMath::Abs(KeyBounds.Max.Y) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Y) >= ZeroingThreshold);
		const bool bHasZ = (FMath::Abs(KeyBounds.Max.Z) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Z) >= ZeroingThreshold);

		if (!bHasX && !bHasY && !bHasZ)
		{
			// No point in using this over the identity encoding
			CompressTranslation_Identity(TranslationData);
			return;
		}

		// Write the header out
		const int32 Header = MakeHeader(NumKeys, ACF_IntervalFixed32NoW, (bHasX ? 1 : 0) | ((bHasY ? 1 : 0)<<1) | ((bHasZ ? 1 : 0)<<2));
		AppendBytes(&Header, sizeof(Header));

		// Write the bounds out
		float Mins[3];
		float Ranges[3];
		FVector3f Range(KeyBounds.Max - KeyBounds.Min);
		Mins[0] = KeyBounds.Min.X;
		Mins[1] = KeyBounds.Min.Y;
		Mins[2] = KeyBounds.Min.Z;
		Ranges[0] = Range.X;
		Ranges[1] = Range.Y;
		Ranges[2] = Range.Z;
		if (bHasX)
		{
			AppendBytes(Mins + 0, sizeof(float));
			AppendBytes(Ranges + 0, sizeof(float));
		}
		else
		{
			Ranges[0] = Mins[0] = 0.0f;
		}

		if (bHasY)
		{
			AppendBytes(Mins + 1, sizeof(float));
			AppendBytes(Ranges + 1, sizeof(float));
		}
		else
		{
			Ranges[1] = Mins[1] = 0.0f;
		}

		if (bHasZ)
		{
			AppendBytes(Mins + 2, sizeof(float));
			AppendBytes(Ranges + 2, sizeof(float));
		}
		else
		{
			Ranges[2] = Mins[2] = 0.0f;
		}

		// Write the keys out
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FVector3f& V = TranslationData.PosKeys[i];
			const FVectorIntervalFixed32NoW Compressor(V, Mins, Ranges);
			AppendBytes(&Compressor, sizeof(Compressor));

			// Decompress and update the error stats
			FVector3f DecompressedV;
			Compressor.ToVector(DecompressedV, Mins, Ranges);

			const float Error = (DecompressedV - V).Size();
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
	}

	template<typename T>
	static FBox CalculateQuatACF96Bounds(const UE::Math::TQuat<T>* Points, int32 NumPoints)
	{
		FBox Results(ForceInitToZero);

		for (int32 i = 0; i < NumPoints; ++i)
		{
			const FQuatFloat96NoW Converter(Points[i]);

			Results += FVector(Converter.X, Converter.Y, Converter.Z);
		}


		return Results;
	}

	void CompressRotation_Identity(const FRotationTrack& RotationData)
	{
		// Compute the error when using this compression type (how far off from identity are they?)
		const int32 NumKeys = RotationData.RotKeys.Num();
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const float Error = FQuat4f::ErrorAutoNormalize(RotationData.RotKeys[i], FQuat4f::Identity);
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
		ActualCompressionMode = ACF_Identity;

		// Add nothing to compressed bytes; this type gets flagged extra-special, back at the offset table
	}

	template <typename CompressorType>
	void InnerCompressRotation(const FRotationTrack& RotationData)
	{
		// Write the keys out
		const int32 NumKeys = RotationData.RotKeys.Num();
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FQuat4f& Q = RotationData.RotKeys[i];
			check(Q.IsNormalized());

			// Compress and write out the quaternion
			const CompressorType Compressor(Q);
			AppendBytes(&Compressor, sizeof(Compressor));

			// Decompress and check the error caused by the compression
			FQuat4f DecompressedQ;
			Compressor.ToQuat(DecompressedQ);

			check(DecompressedQ.IsNormalized());
			const float Error = FQuat4f::ErrorAutoNormalize(Q, DecompressedQ);
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
	}

	// Uncompressed packing still drops the W component, storing a rotation in 3 floats (ACF_Float96NoW)
	void CompressRotation_Uncompressed(const FRotationTrack& RotationData)
	{
		const int32 NumKeys = RotationData.RotKeys.Num();

		// Write the header out
		int32 Header = MakeHeader(NumKeys, ACF_Float96NoW, 7);
		AppendBytes(&Header, sizeof(Header));

		// Write the keys out
		InnerCompressRotation<FQuatFloat96NoW>(RotationData);
	}

	void CompressRotation_16_16_16(const FRotationTrack& RotationData, float ZeroingThreshold)
	{
		const int32 NumKeys = RotationData.RotKeys.Num();

		// Determine the bounds
		const FBox KeyBounds = CalculateQuatACF96Bounds(RotationData.RotKeys.GetData(), NumKeys);
		const bool bHasX = (FMath::Abs(KeyBounds.Max.X) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.X) >= ZeroingThreshold);
		const bool bHasY = (FMath::Abs(KeyBounds.Max.Y) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Y) >= ZeroingThreshold);
		const bool bHasZ = (FMath::Abs(KeyBounds.Max.Z) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Z) >= ZeroingThreshold);

		if (!bHasX && !bHasY && !bHasZ)
		{
			// No point in using this over the identity encoding
			CompressRotation_Identity(RotationData);
			return;
		}


		// Write the header out
		const int32 Header = MakeHeader(NumKeys, ACF_Fixed48NoW, (bHasX ? 1 : 0) | ((bHasY ? 1 : 0)<<1) | ((bHasZ ? 1 : 0)<<2));
		AppendBytes(&Header, sizeof(Header));

		// Write the keys for the non-zero components
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FQuat4f& Q = RotationData.RotKeys[i];

			FQuat4f QRenorm(Q);
			if (!bHasX)
			{
				QRenorm.X = 0;
			}
			if (!bHasY)
			{
				QRenorm.Y = 0;
			}
			if (!bHasZ)
			{
				QRenorm.Z = 0;
			}
			QRenorm.Normalize();

			const FQuatFloat96NoW V(QRenorm);

			uint16 X = 0;
			uint16 Y = 0;
			uint16 Z = 0;

			if (bHasX)
			{
				X = FAnimationCompression_PerTrackUtils::CompressFixed16(V.X);
				AppendBytes(&X, sizeof(X));
			}
			if (bHasY)
			{
				Y = FAnimationCompression_PerTrackUtils::CompressFixed16(V.Y);
				AppendBytes(&Y, sizeof(Y));
			}
			if (bHasZ)
			{
				Z = FAnimationCompression_PerTrackUtils::CompressFixed16(V.Z);
				AppendBytes(&Z, sizeof(Z));
			}

			FQuatFloat96NoW Decompressor;
			Decompressor.X = bHasX ? FAnimationCompression_PerTrackUtils::DecompressFixed16<0>(X) : 0.0f;
			Decompressor.Y = bHasY ? FAnimationCompression_PerTrackUtils::DecompressFixed16<0>(Y) : 0.0f;
			Decompressor.Z = bHasZ ? FAnimationCompression_PerTrackUtils::DecompressFixed16<0>(Z) : 0.0f;

			FQuat4f DecompressedQ;
			Decompressor.ToQuat(DecompressedQ);

			if (!DecompressedQ.IsNormalized())
			{
				UE_LOGF(LogAnimationCompression, Log, "Error: Loss of normalization!");
				UE_LOGF(LogAnimationCompression, Log, "  Track: %i, Key: %i", 0, i);
				UE_LOGF(LogAnimationCompression, Log, "  Q : %ls", *Q.ToString());
				UE_LOGF(LogAnimationCompression, Log, "  Q': %ls", *DecompressedQ.ToString());
				UE_LOGF(LogAnimationCompression, Log, " XYZ: %i, %i, %i", X, Y, Z);
			}

			check(DecompressedQ.IsNormalized());
			const float Error = FQuat4f::ErrorAutoNormalize(Q, DecompressedQ);
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
	}

	void CompressRotation_11_11_10(const FRotationTrack& RotationData, float ZeroingThreshold)
	{
		const int32 NumKeys = RotationData.RotKeys.Num();

		// Determine the bounds
		const FBox KeyBounds = CalculateQuatACF96Bounds(RotationData.RotKeys.GetData(), NumKeys);
		FVector3f Range(KeyBounds.Max - KeyBounds.Min);

		const bool bHasX = (FMath::Abs(KeyBounds.Max.X) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.X) >= ZeroingThreshold);
		const bool bHasY = (FMath::Abs(KeyBounds.Max.Y) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Y) >= ZeroingThreshold);
		const bool bHasZ = (FMath::Abs(KeyBounds.Max.Z) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Z) >= ZeroingThreshold);

		if ((!bHasX && !bHasY && !bHasZ) || (Range.SizeSquared() > 16.0f))
		{
			// If there are no components, then there is no point in using this over the identity encoding
			// If the range is insane, error out early (error metric will be high)
			CompressRotation_Identity(RotationData);
			return;
		}

		// Write the header out
		const int32 Header = MakeHeader(NumKeys, ACF_IntervalFixed32NoW, (bHasX ? 1 : 0) | ((bHasY ? 1 : 0)<<1) | ((bHasZ ? 1 : 0)<<2));
		AppendBytes(&Header, sizeof(Header));

		// Write the bounds out
		float Mins[3];
		float Ranges[3];
		Mins[0] = KeyBounds.Min.X;
		Mins[1] = KeyBounds.Min.Y;
		Mins[2] = KeyBounds.Min.Z;
		Ranges[0] = Range.X;
		Ranges[1] = Range.Y;
		Ranges[2] = Range.Z;
		if (bHasX)
		{
			AppendBytes(Mins + 0, sizeof(float));
			AppendBytes(Ranges + 0, sizeof(float));
		}
		else
		{
			Ranges[0] = Mins[0] = 0.0f;
		}

		if (bHasY)
		{
			AppendBytes(Mins + 1, sizeof(float));
			AppendBytes(Ranges + 1, sizeof(float));
		}
		else
		{
			Ranges[1] = Mins[1] = 0.0f;
		}

		if (bHasZ)
		{
			AppendBytes(Mins + 2, sizeof(float));
			AppendBytes(Ranges + 2, sizeof(float));
		}
		else
		{
			Ranges[2] = Mins[2] = 0.0f;
		}

		// Write the keys out
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FQuat4f& Q = RotationData.RotKeys[i];

			FQuat4f QRenorm(Q);
			if (!bHasX)
			{
				QRenorm.X = 0;
			}
			if (!bHasY)
			{
				QRenorm.Y = 0;
			}
			if (!bHasZ)
			{
				QRenorm.Z = 0;
			}
			QRenorm.Normalize();


			// Compress and write out the quaternion
			const FQuatIntervalFixed32NoW Compressor(QRenorm, Mins, Ranges);
			AppendBytes(&Compressor, sizeof(Compressor));

			// Decompress and check the error caused by the compression
			FQuat4f DecompressedQ;
			Compressor.ToQuat(DecompressedQ, Mins, Ranges);

			if (!DecompressedQ.IsNormalized())
			{
				UE_LOGF(LogAnimationCompression, Log, "Error: Loss of normalization!");
				UE_LOGF(LogAnimationCompression, Log, "  Track: %i, Key: %i", 0, i);
				UE_LOGF(LogAnimationCompression, Log, "  Q : %ls", *Q.ToString());
				UE_LOGF(LogAnimationCompression, Log, "  Q': %ls", *DecompressedQ.ToString());
				UE_LOGF(LogAnimationCompression, Log, " XYZ: %f, %f, %f, %f", QRenorm.X, QRenorm.Y, QRenorm.Z, QRenorm.W);
				UE_LOGF(LogAnimationCompression, Log, " Mins(%f, %f, %f)   Maxs(%f, %f,%f)", KeyBounds.Min.X, KeyBounds.Min.Y, KeyBounds.Min.Z, KeyBounds.Max.X, KeyBounds.Max.Y, KeyBounds.Max.Z);
			}
			check(DecompressedQ.IsNormalized());
			const float Error = FQuat4f::ErrorAutoNormalize(Q, DecompressedQ);
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
	}

	void CompressRotation_Fixed32(const FRotationTrack& RotationData)
	{
		// Write the header out
		const int32 NumKeys = RotationData.RotKeys.Num();
		const int32 Header = MakeHeader(NumKeys, ACF_Fixed32NoW, 7);
		AppendBytes(&Header, sizeof(Header));

		// Write the keys out
		InnerCompressRotation<FQuatFixed32NoW>(RotationData);
	}

	void CompressRotation_Float32(const FRotationTrack& RotationData)
	{
		// Write the header out
		const int32 NumKeys = RotationData.RotKeys.Num();
		const int32 Header = MakeHeader(NumKeys, ACF_Float32NoW, 7);
		AppendBytes(&Header, sizeof(Header));

		// Write the keys out
		InnerCompressRotation<FQuatFloat32NoW>(RotationData);
	}

	void CompressScale_Identity(const FScaleTrack& ScaleData)
	{
		// Compute the error when using this compression type (how far off from (0,0,0) are they?)
		const int32 NumKeys = ScaleData.ScaleKeys.Num();
		for (int32 i = 0; i < NumKeys; ++i)
		{
			float Error = ScaleData.ScaleKeys[i].Size();
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
		ActualCompressionMode = ACF_Identity;

		// Add nothing to compressed bytes; this type gets flagged extra-special, back at the offset table
	}

	void CompressScale_16_16_16(const FScaleTrack& ScaleData, float ZeroingThreshold)
	{
		const int32 NumKeys = ScaleData.ScaleKeys.Num();

		// Determine the bounds
		const FBox3f KeyBounds(ScaleData.ScaleKeys.GetData(), NumKeys);
		const bool bHasX = (FMath::Abs(KeyBounds.Max.X) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.X) >= ZeroingThreshold);
		const bool bHasY = (FMath::Abs(KeyBounds.Max.Y) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Y) >= ZeroingThreshold);
		const bool bHasZ = (FMath::Abs(KeyBounds.Max.Z) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Z) >= ZeroingThreshold);

		if (!bHasX && !bHasY && !bHasZ)
		{
			// No point in using this over the identity encoding
			CompressScale_Identity(ScaleData);
			return;
		}

		// Write the header out
		const int32 Header = MakeHeader(NumKeys, ACF_Fixed48NoW, (bHasX ? 1 : 0) | ((bHasY ? 1 : 0)<<1) | ((bHasZ ? 1 : 0)<<2));
		AppendBytes(&Header, sizeof(Header));

		// Write the keys for the non-zero components
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FVector3f& V = ScaleData.ScaleKeys[i];

			uint16 X = 0;
			uint16 Y = 0;
			uint16 Z = 0;

			if (bHasX)
			{
				X = FAnimationCompression_PerTrackUtils::CompressFixed16(V.X, AnimationCompressionUtils::LogScale);
				AppendBytes(&X, sizeof(X));
			}
			if (bHasY)
			{
				Y = FAnimationCompression_PerTrackUtils::CompressFixed16(V.Y, AnimationCompressionUtils::LogScale);
				AppendBytes(&Y, sizeof(Y));
			}
			if (bHasZ)
			{
				Z = FAnimationCompression_PerTrackUtils::CompressFixed16(V.Z, AnimationCompressionUtils::LogScale);
				AppendBytes(&Z, sizeof(Z));
			}

			const FVector3f DecompressedV(
				bHasX ? FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(X) : 0.0f,
				bHasY ? FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(Y) : 0.0f,
				bHasZ ? FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(Z) : 0.0f);

			const float Error = (V - DecompressedV).Size();
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
	}

	void CompressScale_Uncompressed(const FScaleTrack& ScaleData, float ZeroingThreshold)
	{
		const int32 NumKeys = ScaleData.ScaleKeys.Num();

		// Determine the bounds
		const FBox3f KeyBounds(ScaleData.ScaleKeys.GetData(), NumKeys);
		const bool bHasX = (FMath::Abs(KeyBounds.Max.X) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.X) >= ZeroingThreshold);
		const bool bHasY = (FMath::Abs(KeyBounds.Max.Y) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Y) >= ZeroingThreshold);
		const bool bHasZ = (FMath::Abs(KeyBounds.Max.Z) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Z) >= ZeroingThreshold);

		if( !bHasX && !bHasY && !bHasZ )
		{
			// No point in using this over the identity encoding
			CompressScale_Identity(ScaleData);
			return;
		}

		// Write the header out
		const int32 Header = MakeHeader(NumKeys, ACF_Float96NoW, (bHasX ? 1 : 0) | ((bHasY ? 1 : 0)<<1) | ((bHasZ ? 1 : 0)<<2));
		AppendBytes(&Header, sizeof(Header));

		// Write the keys out
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FVector3f& V = ScaleData.ScaleKeys[i];
			if( bHasX )
			{
				AppendBytes(&(V.X), sizeof(float));
			}
			if( bHasY )
			{
				AppendBytes(&(V.Y), sizeof(float));
			}
			if( bHasZ )
			{
				AppendBytes(&(V.Z), sizeof(float));
			}
		}

		// No error, it's a perfect encoding
		MaxError = 0.0f;
		SumError = 0.0;
	}

	// Encode a 0..1 interval in 10:11:11 (X and Z swizzled in the 11:11:10 source because Z is more important in most animations)
	// and store an uncompressed bounding box at the start of the track to scale that 0..1 back up
	void CompressScale_10_11_11(const FScaleTrack& ScaleData, float ZeroingThreshold)
	{
		const int32 NumKeys = ScaleData.ScaleKeys.Num();

		// Determine the bounds
		const FBox3f KeyBounds(ScaleData.ScaleKeys.GetData(), NumKeys);
		const bool bHasX = (FMath::Abs(KeyBounds.Max.X) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.X) >= ZeroingThreshold);
		const bool bHasY = (FMath::Abs(KeyBounds.Max.Y) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Y) >= ZeroingThreshold);
		const bool bHasZ = (FMath::Abs(KeyBounds.Max.Z) >= ZeroingThreshold) || (FMath::Abs(KeyBounds.Min.Z) >= ZeroingThreshold);

		if (!bHasX && !bHasY && !bHasZ)
		{
			// No point in using this over the identity encoding
			CompressScale_Identity(ScaleData);
			return;
		}

		// Write the header out
		const int32 Header = MakeHeader(NumKeys, ACF_IntervalFixed32NoW, (bHasX ? 1 : 0) | ((bHasY ? 1 : 0)<<1) | ((bHasZ ? 1 : 0)<<2));
		AppendBytes(&Header, sizeof(Header));

		// Write the bounds out
		float Mins[3];
		float Ranges[3];
		FVector3f Range(KeyBounds.Max - KeyBounds.Min);
		Mins[0] = KeyBounds.Min.X;
		Mins[1] = KeyBounds.Min.Y;
		Mins[2] = KeyBounds.Min.Z;
		Ranges[0] = Range.X;
		Ranges[1] = Range.Y;
		Ranges[2] = Range.Z;
		if (bHasX)
		{
			AppendBytes(Mins + 0, sizeof(float));
			AppendBytes(Ranges + 0, sizeof(float));
		}
		else
		{
			Ranges[0] = Mins[0] = 0.0f;
		}

		if (bHasY)
		{
			AppendBytes(Mins + 1, sizeof(float));
			AppendBytes(Ranges + 1, sizeof(float));
		}
		else
		{
			Ranges[1] = Mins[1] = 0.0f;
		}

		if (bHasZ)
		{
			AppendBytes(Mins + 2, sizeof(float));
			AppendBytes(Ranges + 2, sizeof(float));
		}
		else
		{
			Ranges[2] = Mins[2] = 0.0f;
		}

		// Write the keys out
		for (int32 i = 0; i < NumKeys; ++i)
		{
			const FVector3f& V = ScaleData.ScaleKeys[i];
			const FVectorIntervalFixed32NoW Compressor(V, Mins, Ranges);
			AppendBytes(&Compressor, sizeof(Compressor));

			// Decompress and update the error stats
			FVector3f DecompressedV;
			Compressor.ToVector(DecompressedV, Mins, Ranges);

			const float Error = (DecompressedV - V).Size();
			MaxError = FMath::Max(MaxError, Error);
			SumError += Error;
		}
	}

	/** Helper method for writing out the key->frame mapping table with a given index type */
	template <typename FrameIndexType>
	void EmitKeyToFrameTable(int32 NumFrames, float FramesPerSecond, const TArray<float>& Times)
	{
		PadOutputStream();

		// write the key table
		const int32 NumKeys = Times.Num();
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			// Convert the frame time into a frame index and write it out
			FrameIndexType FrameIndex = (FrameIndexType)FMath::Clamp(FMath::TruncToInt((Times[KeyIndex] * FramesPerSecond) + 0.5f), 0, NumFrames - 1);
			AppendBytes(&FrameIndex, sizeof(FrameIndexType));
		}

		PadOutputStream();
	}

	/** Writes out the key->frame mapping table if it is needed for the current compression type */
	void ProcessKeyToFrameTable(const FPerTrackParams& Params, const TArray<float>& FrameTimes)
	{
		if (bReallyNeedsFrameTable && (CompressedBytes.Num() > 0))
		{
			const int32 NumFrames = Params.NumberOfKeys;
			const float SequenceLength = Params.SequenceLength;
			const float FramesPerSecond = (NumFrames - 1) / SequenceLength;

			if (NumFrames <= 0xFF)
			{
				EmitKeyToFrameTable<uint8>(NumFrames, FramesPerSecond, FrameTimes);
			}
			else
			{
				EmitKeyToFrameTable<uint16>(NumFrames, FramesPerSecond, FrameTimes);
			}
		}
	}

public:
	/** Constructs a compressed track of translation data */
	FPerTrackCompressor(int32 InCompressionType, const FTranslationTrack& TranslationData, const FPerTrackParams& Params)
	{
		Reset();
		bReallyNeedsFrameTable = Params.bIncludeKeyTable && (TranslationData.PosKeys.Num() > 1) && (TranslationData.PosKeys.Num() < Params.NumberOfKeys);

		const int32 MaxSize = TranslationData.Times.Num() * sizeof(FVector3f);
		CompressedBytes.Reset(MaxSize);

		switch (InCompressionType)
		{
		case ACF_Identity:
			CompressTranslation_Identity(TranslationData);
			break;
		case ACF_None:
		case ACF_Float96NoW:
			CompressTranslation_Uncompressed(TranslationData, Params.MaxZeroingThreshold);
			break;
		case ACF_Fixed48NoW:
			CompressTranslation_16_16_16(TranslationData, Params.MaxZeroingThreshold);
			break;
		case ACF_IntervalFixed32NoW:
			CompressTranslation_10_11_11(TranslationData, Params.MaxZeroingThreshold);
			break;
			// The following two formats don't work well for translation (fixed range & low precision)
			//case ACF_Fixed32NoW:
			//case ACF_Float32NoW:
		default:
			UE_LOGF(LogAnimationCompression, Fatal,"Unsupported translation compression format");
			break;
		}

		PadOutputStream();

		ProcessKeyToFrameTable(Params, TranslationData.Times);
	}

	/** Constructs a compressed track of rotation data */
	FPerTrackCompressor(int32 InCompressionType, const FRotationTrack& RotationData, const FPerTrackParams& Params)
	{
		Reset();
		bReallyNeedsFrameTable = Params.bIncludeKeyTable && (RotationData.RotKeys.Num() > 1) && (RotationData.RotKeys.Num() < Params.NumberOfKeys);

		const int32 MaxSize = RotationData.Times.Num() * sizeof(FQuat4f);
		CompressedBytes.Reset(MaxSize);

		switch (InCompressionType)
		{
		case ACF_Identity:
			CompressRotation_Identity(RotationData);
			break;
		case ACF_None:
		case ACF_Float96NoW:
			CompressRotation_Uncompressed(RotationData);
			break;
		case ACF_Fixed48NoW:
			CompressRotation_16_16_16(RotationData, Params.MaxZeroingThreshold);
			break;
		case ACF_IntervalFixed32NoW:
			CompressRotation_11_11_10(RotationData, Params.MaxZeroingThreshold);
			break;
		case ACF_Fixed32NoW:
			CompressRotation_Fixed32(RotationData);
			break;
		case ACF_Float32NoW:
			CompressRotation_Float32(RotationData);
			break;
		default:
			UE_LOGF(LogAnimationCompression, Fatal,"Unsupported rotation compression format");
			break;
		}

		PadOutputStream();

		ProcessKeyToFrameTable(Params, RotationData.Times);
	}

	/** Constructs a compressed track of Scale data */
	FPerTrackCompressor(int32 InCompressionType, const FScaleTrack& ScaleData, const FPerTrackParams& Params)
	{
		Reset();
		bReallyNeedsFrameTable = Params.bIncludeKeyTable && (ScaleData.ScaleKeys.Num() > 1) && (ScaleData.ScaleKeys.Num() < Params.NumberOfKeys);

		switch (InCompressionType)
		{
		case ACF_Identity:
			CompressScale_Identity(ScaleData);
			break;
		case ACF_None:
		case ACF_Float96NoW:
			CompressScale_Uncompressed(ScaleData, Params.MaxZeroingThreshold);
			break;
		case ACF_Fixed48NoW:
			CompressScale_16_16_16(ScaleData, Params.MaxZeroingThreshold);
			break;
		case ACF_IntervalFixed32NoW:
			CompressScale_10_11_11(ScaleData, Params.MaxZeroingThreshold);
			break;
			// The following two formats don't work well for Scale (fixed range & low precision)
			//case ACF_Fixed32NoW:
			//case ACF_Float32NoW:
		default:
			UE_LOGF(LogAnimationCompression, Fatal,"Unsupported Scale compression format");
			break;
		}

		PadOutputStream();

		ProcessKeyToFrameTable(Params, ScaleData.Times);
	}
};

UAnimCompress_PerTrackCompression::UAnimCompress_PerTrackCompression(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = TEXT("Compress each track independently");
	MaxPosDiffBitwise = 0.007f;
	MaxAngleDiffBitwise = 0.002f;
	MaxScaleDiffBitwise	= 0.0007f;
	MaxZeroingThreshold = 0.0002f;
	ResampledFramerate = 15.0f;
	bResampleAnimation = false;
	MinKeysForResampling = 10;
	bRetarget = false;
	bActuallyFilterLinearKeys = false;
	bUseAdaptiveError = false;
	ParentingDivisor = 1.0f;
	ParentingDivisorExponent = 1.0f;
	TrackHeightBias = 1;
	bUseOverrideForEndEffectors = false;
	bUseAdaptiveError2 = false;
	RotationErrorSourceRatio = 0.8f;
	TranslationErrorSourceRatio = 0.8f;
	ScaleErrorSourceRatio = 0.001f;
	MaxErrorPerTrackRatio = 0.3f;
	PerturbationProbeSize = 0.001f;

	AllowedRotationFormats.Add(ACF_Identity);
	AllowedRotationFormats.Add(ACF_Fixed48NoW);

	AllowedTranslationFormats.Add(ACF_Identity);
	AllowedTranslationFormats.Add(ACF_IntervalFixed32NoW);
	AllowedTranslationFormats.Add(ACF_Fixed48NoW);

	AllowedScaleFormats.Add(ACF_Identity);
	AllowedScaleFormats.Add(ACF_IntervalFixed32NoW);
	AllowedScaleFormats.Add(ACF_Fixed48NoW);
}

#if WITH_EDITOR
void UAnimCompress_PerTrackCompression::CompressUsingUnderlyingCompressor(
	const FCompressibleAnimData& CompressibleAnimData,
	FCompressibleAnimDataResult& OutCompressedData,
	const TArray<FTranslationTrack>& TranslationData,
	const TArray<FRotationTrack>& RotationData,
	const TArray<FScaleTrack>& ScaleData,
	const bool bFinalPass)
{
	// If not doing final pass, then do the RemoveLinearKey version that is less destructive.
	// We're potentially removing whole tracks here, and that doesn't work well with LinearKeyRemoval algorithm.
	if( !bFinalPass )
	{
		UAnimCompress_RemoveLinearKeys::CompressUsingUnderlyingCompressor(
			CompressibleAnimData,
			OutCompressedData,
			TranslationData,
			RotationData,
			ScaleData,
			bFinalPass);
		return;
	}

	// Grab the cache
	check(OutCompressedData.CompressionUserData);
	FPerTrackCachedInfo* Cache = (FPerTrackCachedInfo*)OutCompressedData.CompressionUserData;

	// record the proper runtime decompressor to use
	FUECompressedAnimDataMutable& AnimData = static_cast<FUECompressedAnimDataMutable&>(*OutCompressedData.AnimData);
	AnimData.KeyEncodingFormat = AKF_PerTrackCompression;
	AnimData.RotationCompressionFormat = ACF_Identity;
	AnimData.TranslationCompressionFormat = ACF_Identity;
	AnimData.ScaleCompressionFormat = ACF_Identity;
	AnimationFormat_SetInterfaceLinks(AnimData);

	// Prime the compression buffers
	check(TranslationData.Num() == RotationData.Num());
	const int32 NumTracks = TranslationData.Num();
	const bool bHasScale = ScaleData.Num() > 0;

	AnimData.CompressedTrackOffsets.Empty(NumTracks*2);
	AnimData.CompressedTrackOffsets.AddUninitialized(NumTracks*2);
	AnimData.CompressedScaleOffsets.Empty(0);

	if ( bHasScale )
	{
		AnimData.CompressedScaleOffsets.SetStripSize(1);
		AnimData.CompressedScaleOffsets.AddUninitialized(NumTracks);
	}

	AnimData.CompressedByteStream.Empty();

	// Compress each track independently
	for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
	{
		if (CompressibleAnimData.IsCancelled())
		{
			return;
		}
		// Compression parameters / thresholds
		FPerTrackParams Params;
		Params.NumberOfKeys = CompressibleAnimData.NumberOfKeys;
		Params.SequenceLength = CompressibleAnimData.SequenceLength;
		Params.MaxZeroingThreshold = MaxZeroingThreshold;

		// Determine the local-space error cutoffs
		float MaxPositionErrorCutoff = MaxPosDiffBitwise;
		float MaxAngleErrorCutoff = MaxAngleDiffBitwise;
		float MaxScaleErrorCutoff = MaxScaleDiffBitwise;

		if (bUseAdaptiveError)
		{
			// The height of the track is the distance from an end effector.  It's used to reduce the acceptable error the
			// higher in the skeleton we get, since a higher bone will cause cascading errors everywhere.
			const int32 PureTrackHeight = Cache->TrackHeights[TrackIndex];
			const int32 EffectiveTrackHeight = FMath::Max(0, PureTrackHeight + TrackHeightBias);

			const float Scaler = 1.0f / FMath::Pow(FMath::Max(ParentingDivisor, 1.0f), EffectiveTrackHeight * FMath::Max(0.0f, ParentingDivisorExponent));

			MaxPositionErrorCutoff = FMath::Max<float>(MaxZeroingThreshold, MaxPosDiff * Scaler);
			MaxAngleErrorCutoff = FMath::Max<float>(MaxZeroingThreshold, MaxAngleDiff * Scaler);
			MaxScaleErrorCutoff = FMath::Max<float>(MaxZeroingThreshold, MaxScaleDiff * Scaler);

			if (bUseOverrideForEndEffectors && (PureTrackHeight == 0))
			{
				MaxPositionErrorCutoff = MinEffectorDiff;
			}
		}
		else if (bUseAdaptiveError2)
		{
			const FAnimPerturbationError& TrackError = Cache->PerTrackErrors[TrackIndex];

			float ThresholdT_DueR = (TrackError.MaxErrorInTransDueToRot > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInTransDueToRot) : 1.0f;
			float ThresholdT_DueT = (TrackError.MaxErrorInTransDueToTrans > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInTransDueToTrans) : 1.0f;
			float ThresholdT_DueS = (TrackError.MaxErrorInTransDueToScale > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInTransDueToScale) : 1.0f;

			//@TODO: Mixing spaces (target angle error is in radians, perturbation is in quaternion component units)
			float ThresholdR_DueR = (TrackError.MaxErrorInRotDueToRot > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInRotDueToRot) : 1.0f;
			float ThresholdR_DueT = (TrackError.MaxErrorInRotDueToTrans > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInRotDueToTrans) : 1.0f;
			float ThresholdR_DueS = (TrackError.MaxErrorInRotDueToScale > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInRotDueToScale) : 1.0f;

			// these values are not used, so I don't think we should calculate?
// 			float ThresholdS_DueR = (TrackError.MaxErrorInScaleDueToRot > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInScaleDueToRot) : 1.0f;
// 			float ThresholdS_DueT = (TrackError.MaxErrorInScaleDueToTrans > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInScaleDueToTrans) : 1.0f;
// 			float ThresholdS_DueS = (TrackError.MaxErrorInScaleDueToScale > UE_SMALL_NUMBER) ? (PerturbationProbeSize / TrackError.MaxErrorInScaleDueToScale) : 1.0f;

			// @Todo fix the error - this doesn't make sense
			MaxAngleErrorCutoff = FMath::Min(MaxAngleDiffBitwise, MaxErrorPerTrackRatio * MaxAngleDiff * FMath::Lerp(ThresholdR_DueR, ThresholdT_DueR, RotationErrorSourceRatio));
			MaxPositionErrorCutoff = FMath::Min(MaxPosDiffBitwise, MaxErrorPerTrackRatio * MaxPosDiff * FMath::Lerp(ThresholdR_DueT, ThresholdT_DueT, TranslationErrorSourceRatio));
			MaxScaleErrorCutoff = FMath::Min(MaxScaleDiffBitwise, MaxErrorPerTrackRatio * MaxScaleDiff * FMath::Lerp(ThresholdR_DueS, ThresholdT_DueS, ScaleErrorSourceRatio));
		}

		// Start compressing translation using a totally lossless float32x3
		const FTranslationTrack& TranslationTrack = TranslationData[TrackIndex];

		Params.bIncludeKeyTable = bActuallyFilterLinearKeys && !FAnimationUtils::HasUniformKeySpacing(CompressibleAnimData.NumberOfKeys, TranslationTrack.Times);
		FPerTrackCompressor BestTranslation(ACF_Float96NoW, TranslationTrack, Params);

		// Try the other translation formats
		for (int32 FormatIndex = 0; FormatIndex < AllowedTranslationFormats.Num(); ++FormatIndex)
		{
			FPerTrackCompressor TrialCompression(AllowedTranslationFormats[FormatIndex], TranslationTrack, Params);

			if (TrialCompression.MaxError <= MaxPositionErrorCutoff)
			{
				// Swap if it's smaller or equal-sized but lower-max-error
				const int32 BytesSaved = BestTranslation.CompressedBytes.Num() - TrialCompression.CompressedBytes.Num();
				const bool bIsImprovement = (BytesSaved > 0) || ((BytesSaved == 0) && (TrialCompression.MaxError < BestTranslation.MaxError));

				if (bIsImprovement)
				{
					BestTranslation = TrialCompression;
				}
			}
		}

		// Start compressing rotation, first using lossless float32x3
		const FRotationTrack& RotationTrack = RotationData[TrackIndex];

		Params.bIncludeKeyTable = bActuallyFilterLinearKeys && !FAnimationUtils::HasUniformKeySpacing(CompressibleAnimData.NumberOfKeys, RotationTrack.Times);
		FPerTrackCompressor BestRotation(ACF_Float96NoW, RotationTrack, Params);

		//bool bLeaveRotationUncompressed = (RotationTrack.Times.Num() <= 1) && (GHighQualityEmptyTracks != 0);
		// Try the other rotation formats
		//if (!bLeaveRotationUncompressed)
		{
			for (int32 FormatIndex = 0; FormatIndex < AllowedRotationFormats.Num(); ++FormatIndex)
			{
				FPerTrackCompressor TrialCompression(AllowedRotationFormats[FormatIndex], RotationTrack, Params);

				if (TrialCompression.MaxError <= MaxAngleErrorCutoff)
				{
					// Swap if it's smaller or equal-sized but lower-max-error
					const int32 BytesSaved = BestRotation.CompressedBytes.Num() - TrialCompression.CompressedBytes.Num();
					const bool bIsImprovement = (BytesSaved > 0) || ((BytesSaved == 0) && (TrialCompression.MaxError < BestRotation.MaxError));

					if (bIsImprovement)
					{
						BestRotation = TrialCompression;
					}
				}
			}
		}

		// Start compressing Scale, first using lossless float32x3
		if (bHasScale)
		{
			const FScaleTrack& ScaleTrack = ScaleData[TrackIndex];

			Params.bIncludeKeyTable = bActuallyFilterLinearKeys && !FAnimationUtils::HasUniformKeySpacing(CompressibleAnimData.NumberOfKeys, ScaleTrack.Times);
			FPerTrackCompressor BestScale(ACF_Float96NoW, ScaleTrack, Params);

			//bool bLeaveScaleUncompressed = (ScaleTrack.Times.Num() <= 1) && (GHighQualityEmptyTracks != 0);
			// Try the other Scale formats
			//if (!bLeaveScaleUncompressed)
			{
				for (int32 FormatIndex = 0; FormatIndex < AllowedScaleFormats.Num(); ++FormatIndex)
				{
					FPerTrackCompressor TrialCompression(AllowedScaleFormats[FormatIndex], ScaleTrack, Params);

					if (TrialCompression.MaxError <= MaxAngleErrorCutoff)
					{
						// Swap if it's smaller or equal-sized but lower-max-error
						const int32 BytesSaved = BestScale.CompressedBytes.Num() - TrialCompression.CompressedBytes.Num();
						const bool bIsImprovement = (BytesSaved > 0) || ((BytesSaved == 0) && (TrialCompression.MaxError < BestScale.MaxError));

						if (bIsImprovement)
						{
							BestScale = TrialCompression;
						}
					}
				}
			}

			int32 ScaleOffset = INDEX_NONE;
			if (BestScale.CompressedBytes.Num() > 0)
			{
				check(BestScale.ActualCompressionMode < ACF_MAX);
				ScaleOffset = AnimData.CompressedByteStream.Num();
				AnimData.CompressedByteStream.Append(BestScale.CompressedBytes);
			}
			AnimData.CompressedScaleOffsets.SetOffsetData(TrackIndex, 0, ScaleOffset);
		}

		// Now write out compression and translation frames into the stream
		int32 TranslationOffset = INDEX_NONE;
		if (BestTranslation.CompressedBytes.Num() > 0 )
		{
			check(BestTranslation.ActualCompressionMode < ACF_MAX);
			TranslationOffset = AnimData.CompressedByteStream.Num();
			AnimData.CompressedByteStream.Append(BestTranslation.CompressedBytes);
		}
		AnimData.CompressedTrackOffsets[TrackIndex*2 + 0] = TranslationOffset;

		int32 RotationOffset = INDEX_NONE;
		if (BestRotation.CompressedBytes.Num() > 0)
		{
			check(BestRotation.ActualCompressionMode < ACF_MAX);
			RotationOffset = AnimData.CompressedByteStream.Num();
			AnimData.CompressedByteStream.Append(BestRotation.CompressedBytes);
		}
		AnimData.CompressedTrackOffsets[TrackIndex*2 + 1] = RotationOffset;
	
#if 0
		// This block outputs information about each individual track during compression, which is useful for debugging the compressors
		UE_LOGF(LogAnimationCompression, Warning, "   Compressed track %i, Trans=%ls_%i (#keys=%i, err=%f), Rot=%ls_%i (#keys=%i, err=%f)  (height=%i max pos=%f, angle=%f)", 
			TrackIndex,
			*FAnimationUtils::GetAnimationCompressionFormatString(static_cast<AnimationCompressionFormat>(BestTranslation.ActualCompressionMode)),
			BestTranslation.ActualCompressionMode != ACF_Identity ? ( ( *( (const int32*)BestTranslation.CompressedBytes.GetTypedData() ) ) >> 24) & 0xF : 0,
			TranslationTrack.PosKeys.Num(),
			BestTranslation.MaxError,
			*FAnimationUtils::GetAnimationCompressionFormatString(static_cast<AnimationCompressionFormat>(BestRotation.ActualCompressionMode)),
			BestRotation.ActualCompressionMode != ACF_Identity ? ( ( *( (const int32*)BestRotation.CompressedBytes.GetTypedData() ) ) >> 24) & 0xF : 0,
			RotationTrack.RotKeys.Num(),
			BestRotation.MaxError,
			(bUseAdaptiveError)? (Cache->TrackHeights(TrackIndex)): -1,
			MaxPositionErrorCutoff,
			MaxAngleErrorCutoff
			);
#endif
	}
}

/** Resamples a track of position keys */
void ResamplePositionKeys(
	FTranslationTrack& Track, 
	float StartTime,
	float IntervalTime)
{
	const int32 KeyCount = Track.Times.Num();

	// Oddness about the original data: 30 keys will have times from 0..1 *inclusive*, and 30 Hz isn't
	// This means the key spacing needs a boost
	if (KeyCount > 1)
	{
		IntervalTime = IntervalTime * (KeyCount / (float)(KeyCount - 1));
	}

	check(Track.Times.Num() == Track.PosKeys.Num());

	TArray<FVector3f> NewPosKeys;
	TArray<float> NewTimes;

	NewTimes.Empty(KeyCount);
	NewPosKeys.Empty(KeyCount);

	float FinalTime = Track.Times[KeyCount - 1];

	// step through and retain the desired interval
	int32 CachedIndex = 0;

	float Time = StartTime;
	while (Time <= FinalTime)
	{
		// Find the bracketing current keys
		if (CachedIndex < KeyCount - 1)
		{
			while ((CachedIndex < KeyCount - 1) && (Track.Times[CachedIndex+1] < Time))
			{
				CachedIndex++;
			}
		}

		FVector3f Value;

		check(Track.Times[CachedIndex] <= Time);
		if (CachedIndex + 1 < KeyCount)
		{
			check(Track.Times[CachedIndex+1] >= Time);

			FVector3f A = Track.PosKeys[CachedIndex];
			FVector3f B = Track.PosKeys[CachedIndex + 1];

			float Alpha = (Time - Track.Times[CachedIndex]) / (Track.Times[CachedIndex+1] - Track.Times[CachedIndex]);
			Value = FMath::Lerp(A, B, Alpha);
		}
		else
		{
			Value = Track.PosKeys[CachedIndex];
		}

		NewPosKeys.Add(Value);
		NewTimes.Add(Time);

		Time += IntervalTime;
	}

	NewTimes.Shrink();
	NewPosKeys.Shrink();

	Track.Times = NewTimes;
	Track.PosKeys = NewPosKeys;
}

/** Resamples a track of Scale keys */
void ResampleScaleKeys(
	FScaleTrack& Track, 
	float StartTime,
	float IntervalTime)
{
	const int32 KeyCount = Track.Times.Num();

	// Oddness about the original data: 30 keys will have times from 0..1 *inclusive*, and 30 Hz isn't
	// This means the key spacing needs a boost
	if (KeyCount > 1)
	{
		IntervalTime = IntervalTime * (KeyCount / (float)(KeyCount - 1));
	}

	check(Track.Times.Num() == Track.ScaleKeys.Num());

	TArray<FVector3f> NewScaleKeys;
	TArray<float> NewTimes;

	NewTimes.Empty(KeyCount);
	NewScaleKeys.Empty(KeyCount);

	float FinalTime = Track.Times[KeyCount - 1];

	// step through and retain the desired interval
	int32 CachedIndex = 0;

	float Time = StartTime;
	while (Time <= FinalTime)
	{
		// Find the bracketing current keys
		if (CachedIndex < KeyCount - 1)
		{
			while ((CachedIndex < KeyCount - 1) && (Track.Times[CachedIndex+1] < Time))
			{
				CachedIndex++;
			}
		}

		FVector3f Value;

		check(Track.Times[CachedIndex] <= Time);
		if (CachedIndex + 1 < KeyCount)
		{
			check(Track.Times[CachedIndex+1] >= Time);

			FVector3f A = Track.ScaleKeys[CachedIndex];
			FVector3f B = Track.ScaleKeys[CachedIndex + 1];

			float Alpha = (Time - Track.Times[CachedIndex]) / (Track.Times[CachedIndex+1] - Track.Times[CachedIndex]);
			Value = FMath::Lerp(A, B, Alpha);
		}
		else
		{
			Value = Track.ScaleKeys[CachedIndex];
		}

		NewScaleKeys.Add(Value);
		NewTimes.Add(Time);

		Time += IntervalTime;
	}

	NewTimes.Shrink();
	NewScaleKeys.Shrink();

	Track.Times = NewTimes;
	Track.ScaleKeys = NewScaleKeys;
}
/**
 * Resamples a track of rotation keys
 */
void ResampleRotationKeys(
	FRotationTrack& Track,
	float StartTime,
	float IntervalTime)
{
	const int32 KeyCount = Track.Times.Num();
	check(Track.Times.Num() == Track.RotKeys.Num());

	// Oddness about the original data: 30 keys will have times from 0..1 *inclusive*, and 30 Hz isn't
	// This means the key spacing needs a boost
	if (KeyCount > 1)
	{
		IntervalTime = IntervalTime * (KeyCount / (float)(KeyCount - 1));
	}

	TArray<FQuat4f> NewRotKeys;
	TArray<float> NewTimes;

	NewTimes.Empty(KeyCount);
	NewRotKeys.Empty(KeyCount);

	float FinalTime = Track.Times[KeyCount - 1];

	// step through and retain the desired interval
	int32 CachedIndex = 0;

	float Time = StartTime;
	while (Time <= FinalTime)
	{
		// Find the bracketing current keys
		if (CachedIndex < KeyCount - 1)
		{
			while ((CachedIndex < KeyCount - 1) && (Track.Times[CachedIndex+1] < Time))
			{
				CachedIndex++;
			}
		}

		FQuat4f Value;

		check(Track.Times[CachedIndex] <= Time);
		if (CachedIndex + 1 < KeyCount)
		{
			check(Track.Times[CachedIndex+1] >= Time);

			FQuat4f A = Track.RotKeys[CachedIndex];
			FQuat4f B = Track.RotKeys[CachedIndex + 1];

			float Alpha = (Time - Track.Times[CachedIndex]) / (Track.Times[CachedIndex+1] - Track.Times[CachedIndex]);
			Value = FMath::Lerp(A, B, Alpha);
			Value.Normalize();
		}
		else
		{
			Value = Track.RotKeys[CachedIndex];
		}

		NewRotKeys.Add(Value);
		NewTimes.Add(Time);

		Time += IntervalTime;
	}

	NewTimes.Shrink();
	NewRotKeys.Shrink();

	Track.Times = NewTimes;
	Track.RotKeys = NewRotKeys;
}




void ResampleKeys(
	TArray<FTranslationTrack>& PositionTracks, 
	TArray<FRotationTrack>& RotationTracks,
	TArray<FScaleTrack>& ScaleTracks,
	float Interval,
	float Time0 = 0.0f)
{
	check(PositionTracks.Num() == RotationTracks.Num());
	check((Time0 >= 0.0f) && (Interval > 0.0f));
	const bool bHasScaleTracks = ScaleTracks.Num() > 0;

	for (int32 TrackIndex = 0; TrackIndex < PositionTracks.Num(); ++TrackIndex)
	{
		ResamplePositionKeys(PositionTracks[TrackIndex], Time0, Interval);
		ResampleRotationKeys(RotationTracks[TrackIndex], Time0, Interval);
		if (bHasScaleTracks)
		{
			ResampleScaleKeys(ScaleTracks[TrackIndex], Time0, Interval);
		}
	}
}




void* UAnimCompress_PerTrackCompression::FilterBeforeMainKeyRemoval(
	const FCompressibleAnimData& CompressibleAnimData,
	TArray<FTranslationTrack>& TranslationData,
	TArray<FRotationTrack>& RotationData, 
	TArray<FScaleTrack>& ScaleData)
{
	const int32 NumTracks = TranslationData.Num();

	// Downsample the keys if enabled
	if ((CompressibleAnimData.NumberOfKeys >= MinKeysForResampling) && bResampleAnimation)
	{
		if(CompressibleAnimData.SequenceLength > 0)
		{
			//Make sure we aren't going to oversample the original animation
			const float CurrentFramerate = (CompressibleAnimData.NumberOfKeys - 1) / CompressibleAnimData.SequenceLength;
			if (CurrentFramerate > ResampledFramerate)
			{
				ResampleKeys(TranslationData, RotationData, ScaleData, 1.0f / ResampledFramerate, 0.0f);
			}
		}
	}

	// Create the cache

	FPerTrackCachedInfo* Cache = new FPerTrackCachedInfo();
	
	// Calculate how far each track is from controlling an end effector
	if (bUseAdaptiveError)
	{
		FAnimationUtils::CalculateTrackHeights(CompressibleAnimData, NumTracks, /*OUT*/ Cache->TrackHeights);
	}

	// Find out how a small change affects the maximum error in the end effectors
	if (bUseAdaptiveError2)
	{
		FVector TranslationProbe(PerturbationProbeSize, PerturbationProbeSize, PerturbationProbeSize);
		FQuat RotationProbe(PerturbationProbeSize, PerturbationProbeSize, PerturbationProbeSize, PerturbationProbeSize);
		FVector ScaleProbe(PerturbationProbeSize, PerturbationProbeSize, PerturbationProbeSize);

		FAnimationUtils::TallyErrorsFromPerturbation(
			CompressibleAnimData,
			NumTracks,
			TranslationProbe,
			RotationProbe,
			ScaleProbe,
			/*OUT*/ Cache->PerTrackErrors);
	}

	// remove obviously redundant keys from the source data
	FilterTrivialKeys(TranslationData, RotationData, ScaleData, TRANSLATION_ZEROING_THRESHOLD, QUATERNION_ZEROING_THRESHOLD, SCALE_ZEROING_THRESHOLD);

	return Cache;
}

void UAnimCompress_PerTrackCompression::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();


		// It is an error to set both bUseAdaptiveError and bUseAdaptiveError2 to true at the same time so make sure if 
		// we are enabling one the other is not enabled.
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimCompress_PerTrackCompression, bUseAdaptiveError))
		{
			// We have changed bUseAdaptiveError, bUseAdaptiveError2 can only be true if it was already true
			// and bUseAdaptiveError is false
			bUseAdaptiveError2 = (!bUseAdaptiveError) && bUseAdaptiveError2;
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimCompress_PerTrackCompression, bUseAdaptiveError2))
		{
			// We have changed bUseAdaptiveError2, bUseAdaptiveError can only be true if it was already true
			// and bUseAdaptiveError2 is not true
			bUseAdaptiveError = (!bUseAdaptiveError2) && bUseAdaptiveError;
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimCompress_PerTrackCompression, MaxZeroingThreshold)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UAnimCompress_PerTrackCompression, MaxPosDiffBitwise))
		{
			//MaxZeroingThreshold needs to stay below MaxPosDiffBitwise
			if (MaxZeroingThreshold > MaxPosDiffBitwise)
			{
				MaxZeroingThreshold = MaxPosDiffBitwise;
			}
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimCompress_PerTrackCompression, AllowedScaleFormats))
		{
			for (TEnumAsByte<enum AnimationCompressionFormat>& ScaleFormat : AllowedScaleFormats)
			{
				if (ScaleFormat == ACF_Fixed32NoW || ScaleFormat == ACF_Float32NoW)
				{
					ScaleFormat = ACF_None;
				}
			}
		}
	}
}

bool UAnimCompress_PerTrackCompression::DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult)
{
	if (!FPlatformProperties::HasEditorOnlyData())
	{
		return false;
	}

	ensure((MaxPosDiffBitwise > 0.0f) && (MaxAngleDiffBitwise > 0.0f) && (MaxScaleDiffBitwise > 0.0f) && (MaxZeroingThreshold >= 0.0f));
	ensure(MaxZeroingThreshold <= MaxPosDiffBitwise);
	ensure(!(bUseAdaptiveError2 && bUseAdaptiveError));

	// Compress
	bool bSuccess = UAnimCompress_RemoveLinearKeys::DoReduction(CompressibleAnimData, OutResult);

	// Delete the cache
	if (OutResult.CompressionUserData != nullptr)
	{
		delete (FPerTrackCachedInfo*)OutResult.CompressionUserData;
		OutResult.CompressionUserData = nullptr;
	}
	
	return bSuccess;
}

void WriteEnumArrayToKey(FArchive& Ar, TArray<TEnumAsByte<enum AnimationCompressionFormat> >& EnumArray)
{
	for (TEnumAsByte<enum AnimationCompressionFormat>& EnumVal : EnumArray)
	{
		uint8 Val = EnumVal.GetValue();
		Ar << Val;
	}
}
void UAnimCompress_PerTrackCompression::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);

	Ar << MaxZeroingThreshold;
	Ar << MaxPosDiffBitwise;
	Ar << MaxAngleDiffBitwise;
	Ar << MaxScaleDiffBitwise;
	
	WriteEnumArrayToKey(Ar, AllowedRotationFormats);
	WriteEnumArrayToKey(Ar, AllowedTranslationFormats);
	WriteEnumArrayToKey(Ar, AllowedScaleFormats);
	
	Ar << ResampledFramerate;
	Ar << MinKeysForResampling;
	Ar << TrackHeightBias;
	Ar << ParentingDivisor;
	Ar << ParentingDivisorExponent;
	Ar << RotationErrorSourceRatio;

	Ar << TranslationErrorSourceRatio;
	Ar << ScaleErrorSourceRatio;
	Ar << MaxErrorPerTrackRatio;
	Ar << PerturbationProbeSize;


	uint8 Flags =	MakeBitForFlag(bResampleAnimation, 0) +
					MakeBitForFlag(bUseAdaptiveError, 1) +
					MakeBitForFlag(bUseOverrideForEndEffectors, 2) +
					MakeBitForFlag(bUseAdaptiveError2, 3);
	Ar << Flags;
}
#endif // WITH_EDITOR

void UAnimCompress_PerTrackCompression::DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const
{
	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

#if WITH_EDITOR
	if (AnimData.KeyEncodingFormat == AKF_VariableKeyLerp)
	{
		// While we compress, we also use the decompression code path before our per track data is populated
		Super::DecompressBone(DecompContext, TrackIndex, OutAtom);
		return;
	}
#endif

	// Initialize to identity to set the scale and in case of a missing rotation or translation codec
	OutAtom.SetIdentity();

	// decompress the translation component
	AEFPerTrackCompressionCodec::GetBoneAtomTranslation(OutAtom, DecompContext, TrackIndex);

	// decompress the rotation component
	AEFPerTrackCompressionCodec::GetBoneAtomRotation(OutAtom, DecompContext, TrackIndex);

	// we assume scale keys can be empty, so only extract if we have valid keys
	if (AnimData.CompressedScaleOffsets.IsValid())
	{
		// decompress the rotation component
		AEFPerTrackCompressionCodec::GetBoneAtomScale(OutAtom, DecompContext, TrackIndex);
	}
}

