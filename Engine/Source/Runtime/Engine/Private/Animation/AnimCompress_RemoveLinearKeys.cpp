// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompress_RemoveLinearKeys.cpp: Keyframe reduction algorithm that simply removes every second key.
=============================================================================*/ 

#include "Animation/AnimCompress_RemoveLinearKeys.h"
#include "Animation/AnimSequence.h"
#include "AnimationUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCompress_RemoveLinearKeys)

// Define to 1 to enable timing of the meat of linear key removal done in DoReduction
// The times are non-trivial, but the extra log spam isn't useful if one isn't optimizing DoReduction runtime
#define TIME_LINEAR_KEY_REMOVAL 0

/**
 * Helper function to enforce that the delta between two Quaternions represents
 * the shortest possible rotation angle
 */
template<typename T>
static UE::Math::TQuat<T> EnforceShortestArc(const UE::Math::TQuat<T>& A, const UE::Math::TQuat<T>& B)
{
	const float DotResult = (A | B);
	const float Bias = FMath::FloatSelect(DotResult, 1.0f, -1.0f);
	return B*Bias;
}

/**
 * Helper functions to calculate the delta between two data types.
 * Used in the FilterLinearKeysTemplate function below
 */

/** CalcDelta for FVectors */
float CalcDelta(const FVector3f& A, const FVector3f& B)
{
	return (A - B).Size();
}

/** CalcDelta for FQuat */
float CalcDelta(const FQuat4f& A, const FQuat4f& B)
{
	return FQuat4f::Error(A, B);
}

UAnimCompress_RemoveLinearKeys::UAnimCompress_RemoveLinearKeys(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bNeedsSkeleton = true;
	Description = TEXT("Remove Linear Keys");
	MaxPosDiff = 0.001f;
	MaxAngleDiff = 0.00075f;
	MaxScaleDiff = 0.000001f;
	MaxEffectorDiff = 0.001f;
	MinEffectorDiff = 0.001f;
	EffectorDiffSocket = 0.001f;
	ParentKeyScale = 2.0f;
	bRetarget = true;
	bActuallyFilterLinearKeys = true;
}

#if WITH_EDITORONLY_DATA
int64 UAnimCompress_RemoveLinearKeys::EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const
{
	int64 BaseSize = AnimSequence.GetApproxRawSize();
	if (const IAnimationDataModel* DataModel = AnimSequence.GetDataModel())
	{
		return BaseSize + 3 * DataModel->GetNumberOfKeys() * DataModel->GetNumBoneTracks() * sizeof(FTransform);
	}
	return BaseSize;
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
struct RotationAdapter
{
	typedef FQuat4f KeyType;

	static FTransform UpdateBoneAtom(const FTransform& Atom, const FQuat4f& Component) { return FTransform(FQuat(Component), Atom.GetTranslation(), Atom.GetScale3D()); }
};

struct TranslationAdapter
{
	typedef FVector3f KeyType;

	static FTransform UpdateBoneAtom(const FTransform& Atom, const FVector3f& Component) { return FTransform(Atom.GetRotation(), (FVector)Component, Atom.GetScale3D()); }
};

struct ScaleAdapter
{
	typedef FVector3f KeyType;

	static FTransform UpdateBoneAtom(const FTransform& Atom, const FVector3f& Component) { return FTransform(Atom.GetRotation(), Atom.GetTranslation(), (FVector)Component); }
};

/**
 * Template function to reduce the keys of a given data type.
 * Used to reduce both Translation, Rotation, and Scale keys using the corresponding
 * data types FVector and FQuat
 */
template <typename AdapterType>
void FilterLinearKeysTemplate(
	TArray<typename AdapterType::KeyType>& Keys,
	TArray<float>& Times,
	TArray<FTransform>& BoneAtoms,
	const TArray<float>* ParentTimes,
	const TArray<FTransform>& RawWorldBones,
	const TArray<FTransform>& NewWorldBones,
	const TArray<int32>& TargetBoneIndices,
	int32 NumFrames,
	int32 BoneIndex,
	int32 ParentBoneIndex,
	float ParentScale,
	float MaxDelta,
	float MaxTargetDelta,
	float EffectorDiffSocket,
	const TArray<FBoneData>& BoneData
	)
{
	typedef typename AdapterType::KeyType KeyType;

	const int32 KeyCount = Keys.Num();
	check( Keys.Num() == Times.Num() );
	check( KeyCount >= 1 );
	
	// generate new arrays we will fill with the final keys
	TArray<KeyType> NewKeys;
	TArray<float> NewTimes;
	NewKeys.Reset(KeyCount);
	NewTimes.Reset(KeyCount);

	// Only bother doing anything if we have some keys!
	if(KeyCount > 0)
	{
		int32 LowKey = 0;
		int32 HighKey = KeyCount-1;

		TArray<bool> KnownParentTimes;
		KnownParentTimes.SetNumUninitialized(KeyCount);
		const int32 ParentKeyCount = ParentTimes ? ParentTimes->Num() : 0;
		for (int32 TimeIndex = 0, ParentTimeIndex = 0; TimeIndex < KeyCount; TimeIndex++)
		{
			while ((ParentTimeIndex < ParentKeyCount) && (Times[TimeIndex] > (*ParentTimes)[ParentTimeIndex]))
			{
				ParentTimeIndex++;
			}

			KnownParentTimes[TimeIndex] = (ParentTimeIndex < ParentKeyCount) && (Times[TimeIndex] == (*ParentTimes)[ParentTimeIndex]);
		}

		TArray<FTransform> CachedInvRawBases;
		CachedInvRawBases.SetNumUninitialized(KeyCount);
		for (int32 FrameIndex = 0; FrameIndex < KeyCount; ++FrameIndex)
		{
			const FTransform& RawBase = RawWorldBones[(BoneIndex*NumFrames) + FrameIndex];
			CachedInvRawBases[FrameIndex] = RawBase.Inverse();
		}
		
		// copy the low key (this one is a given)
		NewTimes.Add(Times[0]);
		NewKeys.Add(Keys[0]);

		const FTransform EndEffectorDummyBoneSocket(FQuat::Identity, FVector(END_EFFECTOR_DUMMY_BONE_LENGTH_SOCKET));
		const FTransform EndEffectorDummyBone(FQuat::Identity, FVector(END_EFFECTOR_DUMMY_BONE_LENGTH));

		const float DeltaThreshold = (BoneData[BoneIndex].IsEndEffector() && (BoneData[BoneIndex].bHasSocket || BoneData[BoneIndex].bKeyEndEffector)) ? EffectorDiffSocket : MaxTargetDelta;

		// We will test within a sliding window between LowKey and HighKey.
		// Therefore, we are done when the LowKey exceeds the range
		while (LowKey + 1 < KeyCount)
		{
			int32 GoodHighKey = LowKey + 1;
			int32 BadHighKey = KeyCount;
			
			// bisect until we find the lowest acceptable high key
			while (BadHighKey - GoodHighKey >= 2)
			{
				HighKey = GoodHighKey + (BadHighKey - GoodHighKey) / 2;

				// get the parameters of the window we are testing
				const float LowTime = Times[LowKey];
				const float HighTime = Times[HighKey];
				const KeyType& LowValue = Keys[LowKey];
				const KeyType& HighValue = Keys[HighKey];
				const float Range = HighTime - LowTime;
				const float InvRange = 1.0f/Range;

				// iterate through all interpolated members of the window to
				// compute the error when compared to the original raw values
				float MaxLerpError = 0.0f;
				float MaxTargetError = 0.0f;
				for (int32 TestKey = LowKey+1; TestKey< HighKey; ++TestKey)
				{
					// get the parameters of the member being tested
					float TestTime = Times[TestKey];
					const KeyType& TestValue = Keys[TestKey];

					// compute the proposed, interpolated value for the key
					const float Alpha = (TestTime - LowTime) * InvRange;
					const KeyType LerpValue = AnimationCompressionUtils::Interpolate(LowValue, HighValue, Alpha);

					// compute the error between our interpolated value and the desired value
					float LerpError = CalcDelta(TestValue, LerpValue);

					// if the local-space lerp error is within our tolerances, we will also check the
					// effect this interpolated key will have on our target end effectors
					float TargetError = -1.0f;
					if (LerpError <= MaxDelta)
					{
						// get the raw world transform for this bone (the original world-space position)
						const int32 FrameIndex = TestKey;
						const FTransform& InvRawBase = CachedInvRawBases[FrameIndex];
						
						// generate the proposed local bone atom and transform (local space)
						FTransform ProposedTM = AdapterType::UpdateBoneAtom(BoneAtoms[FrameIndex], LerpValue);

						// convert the proposed local transform to world space using this bone's parent transform
						const FTransform& CurrentParent = ParentBoneIndex != INDEX_NONE ? NewWorldBones[(ParentBoneIndex*NumFrames) + FrameIndex] : FTransform::Identity;
						FTransform ProposedBase = ProposedTM * CurrentParent;
						
						// for each target end effector, compute the error we would introduce with our proposed key
						for (int32 TargetIndex=0; TargetIndex<TargetBoneIndices.Num(); ++TargetIndex)
						{
							// find the offset transform from the raw base to the end effector
							const int32 TargetBoneIndex = TargetBoneIndices[TargetIndex];
							FTransform RawTarget = RawWorldBones[(TargetBoneIndex*NumFrames) + FrameIndex];
							const FTransform RelTM = RawTarget * InvRawBase;

							// forecast where the new end effector would be using our proposed key
							FTransform ProposedTarget = RelTM * ProposedBase;

							// If this is an EndEffector, add a dummy bone to measure the effect of compressing the rotation.
							// Sockets and Key EndEffectors have a longer dummy bone to maintain higher precision.
							if (BoneData[TargetIndex].bHasSocket || BoneData[TargetIndex].bKeyEndEffector)
							{
								ProposedTarget = EndEffectorDummyBoneSocket * ProposedTarget;
								RawTarget = EndEffectorDummyBoneSocket * RawTarget;
							}
							else
							{
								ProposedTarget = EndEffectorDummyBone * ProposedTarget;
								RawTarget = EndEffectorDummyBone * RawTarget;
							}

							// determine the extend of error at the target end effector
							const float ThisError = (ProposedTarget.GetTranslation() - RawTarget.GetTranslation()).Size();
							TargetError = FMath::Max(TargetError, ThisError); 

							// exit early when we encounter a large delta
							const float TargetDeltaThreshold = BoneData[TargetIndex].bHasSocket ? EffectorDiffSocket : DeltaThreshold;
							if( TargetError > TargetDeltaThreshold )
							{ 
								break;
							}
						}
					}

					// If the parent has a key at this time, we'll scale our error values as requested.
					// This increases the odds that we will choose keys on the same frames as our parent bone,
					// making the skeleton more uniform in key distribution.
					if (ParentTimes)
					{
						if (KnownParentTimes[TestKey])
						{
							// our parent has a key at this time, 
							// inflate our perceived error to increase our sensitivity
							// for also retaining a key at this time
							LerpError *= ParentScale;
							TargetError *= ParentScale;
						}
					}
					
					// keep track of the worst errors encountered for both 
					// the local-space 'lerp' error and the end effector drift we will cause
					MaxLerpError = FMath::Max(MaxLerpError, LerpError);
					MaxTargetError = FMath::Max(MaxTargetError, TargetError);

					// exit early if we have failed in this span
					if (MaxLerpError > MaxDelta ||
						MaxTargetError > DeltaThreshold)
					{
						break;
					}
				}

				// determine if the span succeeded. That is, the worst errors found are within tolerances
				if ((MaxLerpError <= MaxDelta) && (MaxTargetError <= DeltaThreshold))
				{
					GoodHighKey = HighKey;
				}
				else
				{
					BadHighKey = HighKey;
				}
			}

			NewTimes.Add(Times[GoodHighKey]);
			NewKeys.Add(Keys[GoodHighKey]);

			LowKey = GoodHighKey;
		}

		// return the new key set to the caller
		Times= NewTimes;
		Keys= NewKeys;
	}
}


void UAnimCompress_RemoveLinearKeys::UpdateWorldBoneTransformTable(
	const FCompressibleAnimData& CompressibleAnimData,
	FCompressibleAnimDataResult& OutCompressedData,
	const TArray<FTransform>& RefPose,
	int32 BoneIndex, // this bone index should be of skeleton, not mesh
	bool UseRaw,
	TArray<FTransform>& OutputWorldBones)
{
	const FBoneData& Bone		= CompressibleAnimData.BoneData[BoneIndex];
	const int32 NumKeys		= CompressibleAnimData.NumberOfKeys;
	const int32 FrameStart		= (BoneIndex*NumKeys);
	const int32 TrackIndex = FAnimationUtils::GetAnimTrackIndexForSkeletonBone(BoneIndex, CompressibleAnimData.TrackToSkeletonMapTable);
	
	check(OutputWorldBones.Num() >= (FrameStart+NumKeys));

	const FFrameRate& SamplingRate = CompressibleAnimData.SampledFrameRate;

	if( TrackIndex != INDEX_NONE )
	{
		// get the local-space bone transforms using the animation solver
		for ( int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex )
		{
			const double Time = SamplingRate.AsSeconds(KeyIndex);
			FTransform LocalAtom;

			FAnimationUtils::ExtractTransformFromCompressionData(CompressibleAnimData, OutCompressedData, Time, TrackIndex, UseRaw, LocalAtom);

			FQuat Rot = LocalAtom.GetRotation();
			LocalAtom.SetRotation(EnforceShortestArc(FQuat::Identity, Rot));
			// Saw some crashes happening with it, so normalize here. 
			LocalAtom.NormalizeRotation();

			OutputWorldBones[(BoneIndex*NumKeys) + KeyIndex] = LocalAtom;
		}
	}
	else
	{
		// get the default rotation and translation from the reference skeleton
		FTransform DefaultTransform;
		FTransform LocalAtom = RefPose[BoneIndex];
		LocalAtom.SetRotation(EnforceShortestArc(FQuat::Identity, LocalAtom.GetRotation()));
		DefaultTransform = LocalAtom;

		// copy the default transformation into the world bone table
		for ( int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex )
		{
			OutputWorldBones[(BoneIndex*NumKeys) + KeyIndex] = DefaultTransform;
		}
	}

	// apply parent transforms to bake into world space. We assume the parent transforms were previously set using this function
	const int32 ParentIndex = Bone.GetParent();
	if (ParentIndex != INDEX_NONE)
	{
		check (ParentIndex < BoneIndex);
		for ( int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex )
		{
			OutputWorldBones[(BoneIndex*NumKeys) + KeyIndex] = OutputWorldBones[(BoneIndex*NumKeys) + KeyIndex] * OutputWorldBones[(ParentIndex*NumKeys) + KeyIndex];
		}
	}
}


void* UAnimCompress_RemoveLinearKeys::FilterBeforeMainKeyRemoval(
	const FCompressibleAnimData& CompressibleAnimData,
	TArray<FTranslationTrack>& TranslationData,
	TArray<FRotationTrack>& RotationData, 
	TArray<FScaleTrack>& ScaleData)
{
	// remove obviously redundant keys from the source data
	FilterTrivialKeys(TranslationData, RotationData, ScaleData, TRANSLATION_ZEROING_THRESHOLD, QUATERNION_ZEROING_THRESHOLD, SCALE_ZEROING_THRESHOLD);
	return nullptr;
}



void UAnimCompress_RemoveLinearKeys::UpdateWorldBoneTransformRange(
	const FCompressibleAnimData& CompressibleAnimData,
	FCompressibleAnimDataResult& OutCompressedData,
	const TArray<FTransform>& RefPose,
	const TArray<FTranslationTrack>& PositionTracks,
	const TArray<FRotationTrack>& RotationTracks,
	const TArray<FScaleTrack>& ScaleTracks,
	int32 StartingBoneIndex, // this bone index should be of skeleton, not mesh
	int32 EndingBoneIndex,// this bone index should be of skeleton, not mesh
	bool UseRaw,
	TArray<FTransform>& OutputWorldBones)
{
	// bitwise compress the tracks into the anim sequence buffers
	// to make sure the data we've compressed so far is ready for solving
	CompressUsingUnderlyingCompressor(
		CompressibleAnimData,
		OutCompressedData,
		PositionTracks,
		RotationTracks,
		ScaleTracks,
		false);

	// build all world-space transforms from this bone to the target end effector we are monitoring
	// all parent transforms have been built already
	for ( int32 Index = StartingBoneIndex; Index <= EndingBoneIndex; ++Index )
	{
		UpdateWorldBoneTransformTable(
			CompressibleAnimData,
			OutCompressedData,
			RefPose,
			Index,
			UseRaw,
			OutputWorldBones);
	}
}

void UAnimCompress_RemoveLinearKeys::UpdateBoneAtomList(
	const FCompressibleAnimData& CompressibleAnimData,
	FCompressibleAnimDataResult& OutCompressedData,
	int32 BoneIndex,
	int32 TrackIndex,
	int32 NumFrames,
	float TimePerFrame,
	TArray<FTransform>& BoneAtoms)
{
	BoneAtoms.Reset(NumFrames);
	const FFrameRate& SamplingRate = CompressibleAnimData.SampledFrameRate;
	for ( int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex )
	{
		const double Time = SamplingRate.AsSeconds(FrameIndex);
		FTransform LocalAtom;
		FAnimationUtils::ExtractTransformFromCompressionData(CompressibleAnimData, OutCompressedData, Time, TrackIndex, false, LocalAtom);

		FQuat Rot = LocalAtom.GetRotation();
		LocalAtom.SetRotation( EnforceShortestArc(FQuat::Identity, Rot) );
		BoneAtoms.Add(LocalAtom);
	}
}

void UAnimCompress_RemoveLinearKeys::ConvertFromRelativeSpace(FCompressibleAnimData& CompressibleAnimData)
{
	// if this is an additive animation, temporarily convert it out of relative-space
	check(CompressibleAnimData.bIsValidAdditive);
	// convert the raw tracks out of additive-space
	const int32 NumTracks = CompressibleAnimData.RawAnimationData.Num();
	for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
	{
		// bone index of skeleton
		int32 const BoneIndex = CompressibleAnimData.TrackToSkeletonMapTable[TrackIndex].BoneTreeIndex;
		bool const bIsRootBone = (BoneIndex == 0);

		const FRawAnimSequenceTrack& BasePoseTrack = CompressibleAnimData.AdditiveBaseAnimationData[TrackIndex];
		FRawAnimSequenceTrack& RawTrack	= CompressibleAnimData.RawAnimationData[TrackIndex];

		// @note: we only extract the first frame, as we don't want to induce motion from the base pose
		// only the motion from the additive data should matter.
		const FVector3f& RefBonePos = BasePoseTrack.PosKeys[0];
		const FQuat4f& RefBoneRotation = BasePoseTrack.RotKeys[0];

		// Transform position keys.
		for (int32 PosIndex = 0; PosIndex < RawTrack.PosKeys.Num(); ++PosIndex)
		{
			RawTrack.PosKeys[PosIndex] += RefBonePos;
		}

		// Transform rotation keys.
		for (int32 RotIndex = 0; RotIndex < RawTrack.RotKeys.Num(); ++RotIndex)
		{
			RawTrack.RotKeys[RotIndex] = RawTrack.RotKeys[RotIndex] * RefBoneRotation;
			RawTrack.RotKeys[RotIndex].Normalize();
		}

		// make sure scale key exists
		if (RawTrack.ScaleKeys.Num() > 0)
		{
			const FVector3f& RefBoneScale = (BasePoseTrack.ScaleKeys.Num() > 0)? BasePoseTrack.ScaleKeys[0] : FVector3f::OneVector;
			for (int32 ScaleIndex = 0; ScaleIndex < RawTrack.ScaleKeys.Num(); ++ScaleIndex)
			{
				RawTrack.ScaleKeys[ScaleIndex] = RefBoneScale * (FVector3f::OneVector + RawTrack.ScaleKeys[ScaleIndex]);
			}
		}
	}
}

void UAnimCompress_RemoveLinearKeys::ConvertToRelativeSpaceBoth(
	FCompressibleAnimData& CompressibleAnimData,
	TArray<FTranslationTrack>& TranslationData,
	TArray<FRotationTrack>& RotationData, 
	TArray<FScaleTrack>& ScaleData)
{
	ConvertToRelativeSpace(CompressibleAnimData);
	ConvertToRelativeSpace(CompressibleAnimData, TranslationData, RotationData, ScaleData);
}

void UAnimCompress_RemoveLinearKeys::ConvertToRelativeSpace(FCompressibleAnimData& CompressibleAnimData) const
{
	// convert the raw tracks back to additive-space
	const int32 NumTracks = CompressibleAnimData.RawAnimationData.Num();
	for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
	{
		const FRawAnimSequenceTrack& BasePoseTrack = CompressibleAnimData.AdditiveBaseAnimationData[TrackIndex];
		FRawAnimSequenceTrack& RawTrack = CompressibleAnimData.RawAnimationData[TrackIndex];

		// @note: we only extract the first frame, as we don't want to induce motion from the base pose
		// only the motion from the additive data should matter.
		const FQuat4f InvRefBoneRotation = BasePoseTrack.RotKeys[0].Inverse();
		const FVector3f InvRefBoneTranslation = -BasePoseTrack.PosKeys[0];

		// transform position keys.
		for (int32 PosIndex = 0; PosIndex < RawTrack.PosKeys.Num(); ++PosIndex)
		{
			RawTrack.PosKeys[PosIndex] += InvRefBoneTranslation;
		}

		// transform rotation keys.
		for (int32 RotIndex = 0; RotIndex < RawTrack.RotKeys.Num(); ++RotIndex)
		{
			RawTrack.RotKeys[RotIndex] = RawTrack.RotKeys[RotIndex] * InvRefBoneRotation;
			RawTrack.RotKeys[RotIndex].Normalize();
		}

		// scale key
		if (RawTrack.ScaleKeys.Num() > 0)
		{
        	const FVector3f& RefBoneScale = (BasePoseTrack.ScaleKeys.Num() > 0)? BasePoseTrack.ScaleKeys[0] : FVector3f::OneVector;
			const FVector3f InvRefBoneScale = (FVector3f)FTransform::GetSafeScaleReciprocal((FVector)RefBoneScale);

			// transform scale keys.
			for (int32 ScaleIndex = 0; ScaleIndex < RawTrack.ScaleKeys.Num(); ++ScaleIndex)
			{
				// to revert scale correctly, you have to - 1.f
				// check AccumulateWithAdditiveScale
				RawTrack.ScaleKeys[ScaleIndex] = (RawTrack.ScaleKeys[ScaleIndex] * InvRefBoneScale) - 1.f;
			}
		}
	}
}

void UAnimCompress_RemoveLinearKeys::ConvertToRelativeSpace(
	const FCompressibleAnimData& CompressibleAnimData,
	TArray<FTranslationTrack>& TranslationData,
	TArray<FRotationTrack>& RotationData,
	TArray<FScaleTrack>& ScaleData) const
{
	// convert the raw tracks back to additive-space
	const int32 NumTracks = CompressibleAnimData.RawAnimationData.Num();
	for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
	{
		const FRawAnimSequenceTrack& BasePoseTrack = CompressibleAnimData.AdditiveBaseAnimationData[TrackIndex];

		// @note: we only extract the first frame, as we don't want to induce motion from the base pose
		// only the motion from the additive data should matter.
		const FQuat4f InvRefBoneRotation = BasePoseTrack.RotKeys[0].Inverse();
		const FVector3f InvRefBoneTranslation = -BasePoseTrack.PosKeys[0];

		// convert the new translation tracks to additive space
		FTranslationTrack& TranslationTrack = TranslationData[TrackIndex];
		for (int32 KeyIndex = 0; KeyIndex < TranslationTrack.PosKeys.Num(); ++KeyIndex)
		{
			TranslationTrack.PosKeys[KeyIndex] += InvRefBoneTranslation;
		}

		// convert the new rotation tracks to additive space
		FRotationTrack& RotationTrack = RotationData[TrackIndex];
		for (int32 KeyIndex = 0; KeyIndex < RotationTrack.RotKeys.Num(); ++KeyIndex)
		{
			RotationTrack.RotKeys[KeyIndex] = RotationTrack.RotKeys[KeyIndex] * InvRefBoneRotation;
			RotationTrack.RotKeys[KeyIndex].Normalize();
		}

		// scale key
		if (ScaleData.Num() > 0)
		{
        	const FVector3f& RefBoneScale = (BasePoseTrack.ScaleKeys.Num() > 0)? BasePoseTrack.ScaleKeys[0] : FVector3f::OneVector;
			const FVector3f InvRefBoneScale = (FVector3f)FTransform::GetSafeScaleReciprocal((FVector)RefBoneScale);

			// convert the new scale tracks to additive space
			FScaleTrack& ScaleTrack = ScaleData[TrackIndex];
			for (int32 KeyIndex = 0; KeyIndex < ScaleTrack.ScaleKeys.Num(); ++KeyIndex)
			{
				ScaleTrack.ScaleKeys[KeyIndex] = (ScaleTrack.ScaleKeys[KeyIndex] * InvRefBoneScale) - 1.f;
			}
		}
	}
}

void UAnimCompress_RemoveLinearKeys::ProcessAnimationTracks(
	const FCompressibleAnimData& CompressibleAnimData,
	FCompressibleAnimDataResult& OutCompressedData,
	TArray<FTranslationTrack>& PositionTracks,
	TArray<FRotationTrack>& RotationTracks, 
	TArray<FScaleTrack>& ScaleTracks)
{
	// extract all the data we'll need about the skeleton and animation sequence
	const int32 NumBones			= CompressibleAnimData.BoneData.Num();
	const int32 NumKeys			= CompressibleAnimData.NumberOfKeys;
	const float SequenceLength	= CompressibleAnimData.SequenceLength;
	const int32 LastFrame = NumKeys-1;
	const float FrameRate = (float)(LastFrame) / SequenceLength;
	const float TimePerFrame = SequenceLength / (float)(LastFrame);

	const TArray<FTransform>& RefPose = CompressibleAnimData.RefLocalPoses;
	const bool bHasScale =  (ScaleTracks.Num() > 0);

	// make sure the parent key scale is properly bound to 1.0 or more
	const float ClampedParentKeyScale = FMath::Max(ParentKeyScale, 1.0f);

	// generate the raw and compressed skeleton in world-space
	TArray<FTransform> RawWorldBones;
	TArray<FTransform> NewWorldBones;
	RawWorldBones.Empty(NumBones * NumKeys);
	NewWorldBones.Empty(NumBones * NumKeys);
	RawWorldBones.AddZeroed(NumBones * NumKeys);
	NewWorldBones.AddZeroed(NumBones * NumKeys);

	// generate an array to hold the indices of our end effectors
	TArray<int32> EndEffectors;
	EndEffectors.Empty(NumBones);

	// Create an array of FTransform to use as a workspace
	TArray<FTransform> BoneAtoms;

	// setup the raw bone transformation and find all end effectors
	for ( int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex )
	{
		if (CompressibleAnimData.IsCancelled())
		{
			return;
		}
		// get the raw world-atoms for this bone
		UpdateWorldBoneTransformTable(
			CompressibleAnimData,
			OutCompressedData,
			RefPose,
			BoneIndex,
			true,
			RawWorldBones);

		// also record all end-effectors we find
		const FBoneData& Bone = CompressibleAnimData.BoneData[BoneIndex];
		if (Bone.IsEndEffector())
		{
			EndEffectors.Add(BoneIndex);
		}
	}

	TArray<int32> TargetBoneIndices;
	// for each bone...
	for ( int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex )
	{
		if (CompressibleAnimData.IsCancelled())
		{
			return;
		}
		const FBoneData& Bone = CompressibleAnimData.BoneData[BoneIndex];
		const int32 ParentBoneIndex = Bone.GetParent();

		const int32 TrackIndex = FAnimationUtils::GetAnimTrackIndexForSkeletonBone(BoneIndex, CompressibleAnimData.TrackToSkeletonMapTable);

		if (TrackIndex != INDEX_NONE)
		{
			// get the tracks we will be editing for this bone
			FRotationTrack& RotTrack = RotationTracks[TrackIndex];
			FTranslationTrack& TransTrack = PositionTracks[TrackIndex];
			const int32 NumRotKeys = RotTrack.RotKeys.Num();
			const int32 NumPosKeys = TransTrack.PosKeys.Num();
			const int32 NumScaleKeys = (bHasScale)? ScaleTracks[TrackIndex].ScaleKeys.Num() : 0;

			check( (NumPosKeys == 1) || (NumRotKeys == 1) || (NumPosKeys == NumRotKeys) );

			// build an array of end effectors we need to monitor
			TargetBoneIndices.Reset(NumBones);

			int32 HighestTargetBoneIndex = BoneIndex;
			int32 FurthestTargetBoneIndex = BoneIndex;
			int32 ShortestChain = 0;
			float OffsetLength= -1.0f;
			for (int32 EffectorIndex=0; EffectorIndex < EndEffectors.Num(); ++EffectorIndex)
			{
				const int32 EffectorBoneIndex = EndEffectors[EffectorIndex];
				const FBoneData& EffectorBoneData = CompressibleAnimData.BoneData[EffectorBoneIndex];

				int32 RootIndex = EffectorBoneData.BonesToRoot.Find(BoneIndex);
				if (RootIndex != INDEX_NONE)
				{
					if (ShortestChain == 0 || (RootIndex+1) < ShortestChain)
					{
						ShortestChain = (RootIndex+1);
					}
					TargetBoneIndices.Add(EffectorBoneIndex);
					HighestTargetBoneIndex = FMath::Max(HighestTargetBoneIndex, EffectorBoneIndex);
					float ChainLength= 0.0f;
					for (long FamilyIndex=0; FamilyIndex < RootIndex; ++FamilyIndex)
					{
						const int32 NextParentBoneIndex= EffectorBoneData.BonesToRoot[FamilyIndex];
						ChainLength += RefPose[NextParentBoneIndex].GetTranslation().Size();
					}

					if (ChainLength > OffsetLength)
					{
						FurthestTargetBoneIndex = EffectorBoneIndex;
						OffsetLength = ChainLength;
					}

				}
			}

			// if requested, retarget the FBoneAtoms towards the target end effectors
			if (bRetarget)
			{
				if (NumScaleKeys > 0 && ParentBoneIndex != INDEX_NONE)
				{
					// update our bone table from the current bone through the last end effector we need to test
					UpdateWorldBoneTransformRange(
						CompressibleAnimData,
						OutCompressedData,
						RefPose,
						PositionTracks,
						RotationTracks,
						ScaleTracks,
						BoneIndex,
						HighestTargetBoneIndex,
						false,
						NewWorldBones);
					
					FScaleTrack& ScaleTrack = ScaleTracks[TrackIndex];

					// adjust all translation keys to align better with the destination
					for ( int32 KeyIndex = 0; KeyIndex < NumScaleKeys; ++KeyIndex )
					{
						FVector3f& Key= ScaleTrack.ScaleKeys[KeyIndex];

						const int32 FrameIndex= FMath::Clamp(KeyIndex, 0, LastFrame);
						const FTransform& NewWorldParent = NewWorldBones[(ParentBoneIndex*NumKeys) + FrameIndex];
						const FTransform& RawWorldChild = RawWorldBones[(BoneIndex*NumKeys) + FrameIndex];
						const FTransform& RelTM = (RawWorldChild.GetRelativeTransform(NewWorldParent));
						const FTransform Delta = FTransform(RelTM);

						Key = (FVector3f)Delta.GetScale3D();
					}
				}
							
				if (NumRotKeys > 0 && ParentBoneIndex != INDEX_NONE)
				{
					if (HighestTargetBoneIndex == BoneIndex)
					{
						for ( int32 KeyIndex = 0; KeyIndex < NumRotKeys; ++KeyIndex )
						{
							FQuat4f& Key = RotTrack.RotKeys[KeyIndex];

							check(ParentBoneIndex != INDEX_NONE);
							const int32 FrameIndex = FMath::Clamp(KeyIndex, 0, LastFrame);
							FTransform NewWorldParent = NewWorldBones[(ParentBoneIndex*NumKeys) + FrameIndex];
							FTransform RawWorldChild = RawWorldBones[(BoneIndex*NumKeys) + FrameIndex];
							const FTransform& RelTM = (RawWorldChild.GetRelativeTransform(NewWorldParent)); 
							FQuat Rot = FTransform(RelTM).GetRotation();

							const FQuat4f& AlignedKey = EnforceShortestArc(Key,FQuat4f(Rot));
							Key = AlignedKey;
						}
					}
					else
					{
						// update our bone table from the current bone through the last end effector we need to test
						UpdateWorldBoneTransformRange(
							CompressibleAnimData,
							OutCompressedData,
							RefPose,
							PositionTracks,
							RotationTracks,
							ScaleTracks,
							BoneIndex,
							HighestTargetBoneIndex,
							false,
							NewWorldBones);
						
						// adjust all rotation keys towards the end effector target
						for ( int32 KeyIndex = 0; KeyIndex < NumRotKeys; ++KeyIndex )
						{
							FQuat4f& Key = RotTrack.RotKeys[KeyIndex];

							const int32 FrameIndex = FMath::Clamp(KeyIndex, 0, LastFrame);

							const FTransform& NewWorldTransform = NewWorldBones[(BoneIndex*NumKeys) + FrameIndex];

							const FTransform& DesiredChildTransform = RawWorldBones[(FurthestTargetBoneIndex*NumKeys) + FrameIndex].GetRelativeTransform(NewWorldTransform);
							const FTransform& CurrentChildTransform = NewWorldBones[(FurthestTargetBoneIndex*NumKeys) + FrameIndex].GetRelativeTransform(NewWorldTransform);

							// find the two vectors which represent the angular error we are trying to correct
							const FVector& CurrentHeading = CurrentChildTransform.GetTranslation();
							const FVector& DesiredHeading = DesiredChildTransform.GetTranslation();

							// if these are valid, we can continue
							if (!CurrentHeading.IsNearlyZero() && !DesiredHeading.IsNearlyZero())
							{
								const float DotResult = CurrentHeading.GetSafeNormal() | DesiredHeading.GetSafeNormal();

								// limit the range we will retarget to something reasonable (~60 degrees)
								if (DotResult < 1.0f && DotResult > 0.5f)
								{
									FQuat4f Adjustment= FQuat4f::FindBetweenVectors((FVector3f)CurrentHeading, (FVector3f)DesiredHeading);
									Adjustment = EnforceShortestArc(FQuat4f::Identity, Adjustment);

									const FVector3f Test = Adjustment.RotateVector((FVector3f)CurrentHeading);
									const float DeltaSqr = (Test - (FVector3f)DesiredHeading).SizeSquared();
									if (DeltaSqr < FMath::Square(0.001f))
									{
										FQuat4f NewKey = Adjustment * Key;
										NewKey.Normalize();

										const FQuat4f& AlignedKey = EnforceShortestArc(Key, NewKey);
										Key = AlignedKey;
									}
								}
							}
						}
					}
				}

				if (NumPosKeys > 0 && ParentBoneIndex != INDEX_NONE)
				{
					// update our bone table from the current bone through the last end effector we need to test
					UpdateWorldBoneTransformRange(
						CompressibleAnimData,
						OutCompressedData,
						RefPose,
						PositionTracks,
						RotationTracks,
						ScaleTracks,
						BoneIndex,
						HighestTargetBoneIndex,
						false,
						NewWorldBones);
					
					// adjust all translation keys to align better with the destination
					for ( int32 KeyIndex = 0; KeyIndex < NumPosKeys; ++KeyIndex )
					{
						FVector3f& Key= TransTrack.PosKeys[KeyIndex];

						const int32 FrameIndex= FMath::Clamp(KeyIndex, 0, LastFrame);
						FTransform NewWorldParent = NewWorldBones[(ParentBoneIndex*NumKeys) + FrameIndex];
						FTransform RawWorldChild = RawWorldBones[(BoneIndex*NumKeys) + FrameIndex];
						const FTransform& RelTM = RawWorldChild.GetRelativeTransform(NewWorldParent);
						const FTransform Delta = FTransform(RelTM);
						ensure (!Delta.ContainsNaN());

						Key = (FVector3f)Delta.GetTranslation();
					}
				}

			}

			// look for a parent track to reference as a guide
			int32 GuideTrackIndex = INDEX_NONE;
			if (ClampedParentKeyScale > 1.0f)
			{
				for (long FamilyIndex=0; (FamilyIndex < Bone.BonesToRoot.Num()) && (GuideTrackIndex == INDEX_NONE); ++FamilyIndex)
				{
					const int32 NextParentBoneIndex= Bone.BonesToRoot[FamilyIndex];

					GuideTrackIndex = FAnimationUtils::GetAnimTrackIndexForSkeletonBone(NextParentBoneIndex, CompressibleAnimData.TrackToSkeletonMapTable);
				}
			}

			// update our bone table from the current bone through the last end effector we need to test
			UpdateWorldBoneTransformRange(
				CompressibleAnimData,
				OutCompressedData,
				RefPose,
				PositionTracks,
				RotationTracks,
				ScaleTracks,
				BoneIndex,
				HighestTargetBoneIndex,
				false,
				NewWorldBones);
			
			// rebuild the BoneAtoms table using the current set of keys
			UpdateBoneAtomList(CompressibleAnimData, OutCompressedData, BoneIndex, TrackIndex, NumKeys, TimePerFrame, BoneAtoms);

			// determine the EndEffectorTolerance. 
			// We use the Maximum value by default, and the Minimum value
			// as we approach the end effectors
			float EndEffectorTolerance = MaxEffectorDiff;
			if (ShortestChain <= 1)
			{
				EndEffectorTolerance = MinEffectorDiff;
			}

			// Determine if a guidance track should be used to aid in choosing keys to retain
			TArray<float>* GuidanceTrack = nullptr;
			float GuidanceScale = 1.0f;
			if (GuideTrackIndex != INDEX_NONE)
			{
				FTranslationTrack& GuideTransTrack = PositionTracks[GuideTrackIndex];
				GuidanceTrack = &GuideTransTrack.Times;
				GuidanceScale = ClampedParentKeyScale;
			}
			
			// if the TargetBoneIndices array is empty, then this bone is an end effector.
			// so we add it to the list to maintain our tolerance checks
			if (TargetBoneIndices.Num() == 0)
			{
				TargetBoneIndices.Add(BoneIndex);
			}

			if (bActuallyFilterLinearKeys)
			{
				if (bHasScale)
				{
					FScaleTrack& ScaleTrack = ScaleTracks[TrackIndex];
					// filter out translations we can approximate through interpolation
					FilterLinearKeysTemplate<ScaleAdapter>(
						ScaleTrack.ScaleKeys, 
						ScaleTrack.Times, 
						BoneAtoms,
						GuidanceTrack, 
						RawWorldBones,
						NewWorldBones,
						TargetBoneIndices,
						NumKeys,
						BoneIndex,
						ParentBoneIndex,
						GuidanceScale, 
						MaxScaleDiff, 
						EndEffectorTolerance,
						EffectorDiffSocket,
						CompressibleAnimData.BoneData);

					// update our bone table from the current bone through the last end effector we need to test
					UpdateWorldBoneTransformRange(
						CompressibleAnimData,
						OutCompressedData,
						RefPose,
						PositionTracks,
						RotationTracks,
						ScaleTracks,
						BoneIndex,
						HighestTargetBoneIndex,
						false,
						NewWorldBones);
					
					// rebuild the BoneAtoms table using the current set of keys
					UpdateBoneAtomList(CompressibleAnimData, OutCompressedData, BoneIndex, TrackIndex, NumKeys, TimePerFrame, BoneAtoms);
				}

				// filter out translations we can approximate through interpolation
				FilterLinearKeysTemplate<TranslationAdapter>(
					TransTrack.PosKeys, 
					TransTrack.Times, 
					BoneAtoms,
					GuidanceTrack, 
					RawWorldBones,
					NewWorldBones,
					TargetBoneIndices,
					NumKeys,
					BoneIndex,
					ParentBoneIndex,
					GuidanceScale, 
					MaxPosDiff, 
					EndEffectorTolerance,
					EffectorDiffSocket,
					CompressibleAnimData.BoneData);

				// update our bone table from the current bone through the last end effector we need to test
				UpdateWorldBoneTransformRange(
					CompressibleAnimData,
					OutCompressedData,
					RefPose,
					PositionTracks,
					RotationTracks,
					ScaleTracks,
					BoneIndex,
					HighestTargetBoneIndex,
					false,
					NewWorldBones);
				
				// rebuild the BoneAtoms table using the current set of keys
				UpdateBoneAtomList(CompressibleAnimData, OutCompressedData, BoneIndex, TrackIndex, NumKeys, TimePerFrame, BoneAtoms);

				// filter out rotations we can approximate through interpolation
				FilterLinearKeysTemplate<RotationAdapter>(
					RotTrack.RotKeys, 
					RotTrack.Times, 
					BoneAtoms,
					GuidanceTrack, 
					RawWorldBones,
					NewWorldBones,
					TargetBoneIndices,
					NumKeys,
					BoneIndex,
					ParentBoneIndex,
					GuidanceScale, 
					MaxAngleDiff, 
					EndEffectorTolerance,
					EffectorDiffSocket,
					CompressibleAnimData.BoneData);
			}
		}

		// make sure the final compressed keys are repesented in our NewWorldBones table
		UpdateWorldBoneTransformRange(
			CompressibleAnimData,
			OutCompressedData,
			RefPose,
			PositionTracks,
			RotationTracks,
			ScaleTracks,
			BoneIndex,
			BoneIndex,
			false,
			NewWorldBones);
	}
};

void UAnimCompress_RemoveLinearKeys::CompressUsingUnderlyingCompressor(
	const FCompressibleAnimData& CompressibleAnimData,
	FCompressibleAnimDataResult& OutCompressedData,
	const TArray<FTranslationTrack>& TranslationData,
	const TArray<FRotationTrack>& RotationData,
	const TArray<FScaleTrack>& ScaleData,
	const bool bFinalPass)
{
	BitwiseCompressAnimationTracks(
		CompressibleAnimData,
		OutCompressedData,
		static_cast<AnimationCompressionFormat>(TranslationCompressionFormat),
		static_cast<AnimationCompressionFormat>(RotationCompressionFormat),
		static_cast<AnimationCompressionFormat>(ScaleCompressionFormat),
		TranslationData,
		RotationData,
		ScaleData,
		true);

	// record the proper runtime decompressor to use
	FUECompressedAnimDataMutable& AnimData = static_cast<FUECompressedAnimDataMutable&>(*OutCompressedData.AnimData);
	AnimData.KeyEncodingFormat = AKF_VariableKeyLerp;
	AnimationFormat_SetInterfaceLinks(AnimData);
}

bool UAnimCompress_RemoveLinearKeys::DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult)
{
#if WITH_EDITORONLY_DATA
	// Only need to do the heavy lifting if it will have some impact
	// One of these will always be true for the base class, but derived classes may choose to turn both off (e.g., in PerTrackCompression)
	const bool bRunningProcessor = bRetarget || bActuallyFilterLinearKeys;

	// If the processor is to be run, then additive animations need to be converted from relative to absolute
	const bool bNeedToConvertBackToAdditive = bRunningProcessor ? CompressibleAnimData.bIsValidAdditive : false;

	TUniquePtr<FCompressibleAnimData> TempConvertedAnimData;

	if (bNeedToConvertBackToAdditive)
	{
		TempConvertedAnimData = MakeUnique<FCompressibleAnimData>(CompressibleAnimData); // duplicate so we can safely convert to additive
		ConvertFromRelativeSpace(*TempConvertedAnimData);
	}

	const FCompressibleAnimData& CompressibleDataToOperateOn = bNeedToConvertBackToAdditive ? *TempConvertedAnimData : CompressibleAnimData;

	// Separate the raw data into tracks and remove trivial tracks (all the same value)
	TArray<FTranslationTrack> TranslationData;
	TArray<FRotationTrack> RotationData;
	TArray<FScaleTrack> ScaleData;
	SeparateRawDataIntoTracks(CompressibleDataToOperateOn.RawAnimationData, CompressibleDataToOperateOn.SequenceLength, TranslationData, RotationData, ScaleData);

	check(OutResult.CompressionUserData == nullptr);
	OutResult.CompressionUserData = FilterBeforeMainKeyRemoval(CompressibleDataToOperateOn, TranslationData, RotationData, ScaleData);

	if (bRunningProcessor)
	{
#if TIME_LINEAR_KEY_REMOVAL
		double TimeStart = FPlatformTime::Seconds();
#endif
		// compress this animation without any key-reduction to prime the codec
		CompressUsingUnderlyingCompressor(
			CompressibleDataToOperateOn,
			OutResult,
			TranslationData,
			RotationData,
			ScaleData,
			false);

		// now remove the keys which can be approximated with linear interpolation
		ProcessAnimationTracks(
			CompressibleDataToOperateOn,
			OutResult,
			TranslationData,
			RotationData, 
			ScaleData);

#if TIME_LINEAR_KEY_REMOVAL
		double ElapsedTime = FPlatformTime::Seconds() - TimeStart;
		UE_LOGF(LogAnimationCompression, Log, "ProcessAnimationTracks time is (%f) seconds",ElapsedTime);
#endif

		if (bNeedToConvertBackToAdditive)
		{
			ConvertToRelativeSpace(CompressibleDataToOperateOn, TranslationData, RotationData, ScaleData);
		}
	}

	// compress the final (possibly key-reduced) tracks into the anim sequence buffers
	CompressUsingUnderlyingCompressor(
		CompressibleAnimData,
		OutResult,
		TranslationData,
		RotationData,
		ScaleData,
		true);

#endif // WITH_EDITORONLY_DATA
	return true;
}

void UAnimCompress_RemoveLinearKeys::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);
	Ar << MaxPosDiff;
	Ar << MaxAngleDiff;
	Ar << MaxScaleDiff;
	Ar << MaxEffectorDiff;
	Ar << MinEffectorDiff;
	Ar << EffectorDiffSocket;
	Ar << ParentKeyScale;
	uint8 Flags =	MakeBitForFlag(bRetarget, 0) +
					MakeBitForFlag(bActuallyFilterLinearKeys, 1);
	Ar << Flags;
}

#endif // WITH_EDITOR

