// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompress_RemoveTrivialKeys.cpp: Removes trivial frames from the raw animation data.
=============================================================================*/ 

#include "Animation/AnimCompress_RemoveTrivialKeys.h"
#include "Animation/AnimSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCompress_RemoveTrivialKeys)

UAnimCompress_RemoveTrivialKeys::UAnimCompress_RemoveTrivialKeys(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = TEXT("Remove Trivial Keys");
	MaxPosDiff = TRANSLATION_ZEROING_THRESHOLD;
	MaxAngleDiff = QUATERNION_ZEROING_THRESHOLD;
	MaxScaleDiff = SCALE_ZEROING_THRESHOLD;
}

#if WITH_EDITOR
bool UAnimCompress_RemoveTrivialKeys::DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult)
{
#if WITH_EDITORONLY_DATA
	// split the filtered data into tracks
	TArray<FTranslationTrack> TranslationData;
	TArray<FRotationTrack> RotationData;
	TArray<FScaleTrack> ScaleData;
	SeparateRawDataIntoTracks( CompressibleAnimData.RawAnimationData, CompressibleAnimData.SequenceLength, TranslationData, RotationData, ScaleData );
	
	// remove obviously redundant keys from the source data
	FilterTrivialKeys(TranslationData, RotationData, ScaleData, MaxPosDiff, MaxAngleDiff, MaxScaleDiff);

	// record the proper runtime decompressor to use
	FUECompressedAnimDataMutable& AnimData = static_cast<FUECompressedAnimDataMutable&>(*OutResult.AnimData);
	AnimData.KeyEncodingFormat = AKF_ConstantKeyLerp;
	AnimationFormat_SetInterfaceLinks(AnimData);

	// bitwise compress the tracks into the anim sequence buffers
	BitwiseCompressAnimationTracks(
		CompressibleAnimData,
		OutResult,
		static_cast<AnimationCompressionFormat>(TranslationCompressionFormat),
		static_cast<AnimationCompressionFormat>(RotationCompressionFormat),
		static_cast<AnimationCompressionFormat>(ScaleCompressionFormat),
		TranslationData,
		RotationData,
		ScaleData);

	// We could be invalid, set the links again
	AnimationFormat_SetInterfaceLinks(AnimData);
#endif // WITH_EDITORONLY_DATA

	return true;
}

void UAnimCompress_RemoveTrivialKeys::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);
	Ar << MaxPosDiff;
	Ar << MaxAngleDiff;
	Ar << MaxScaleDiff;
}

#endif // WITH_EDITOR

