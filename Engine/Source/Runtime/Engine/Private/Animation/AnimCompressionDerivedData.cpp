// Copyright Epic Games, Inc. All Rights Reserved.
#include "Animation/AnimCompressionDerivedData.h"
#include "DerivedDataCacheInterface.h"
#include "Misc/ScopeExit.h"
#include "Stats/Stats.h"
#include "Animation/AnimSequence.h"
#include "AnimationUtils.h"
#include "Animation/AnimCompress.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Animation/AnimationCompressionDerivedData.h"

#if WITH_EDITOR

DECLARE_CYCLE_STAT(TEXT("Anim Compression (Derived Data)"), STAT_AnimCompressionDerivedData, STATGROUP_Anim);

FDerivedDataAnimationCompression::FDerivedDataAnimationCompression(const TCHAR* InTypeName, const FString& InAssetDDCKey)
	: TypeName(InTypeName)
	, AssetDDCKey(InAssetDDCKey)
{

}

FDerivedDataAnimationCompression::~FDerivedDataAnimationCompression()
{
}

const TCHAR* FDerivedDataAnimationCompression::GetVersionString() const
{
	return *UE::Anim::Compression::AnimationCompressionVersionString;
}

FString FDerivedDataAnimationCompression::GetDebugContextString() const
{
	check(DataToCompressPtr.IsValid());
	return DataToCompressPtr->FullName;
}

bool FDerivedDataAnimationCompression::Build( TArray<uint8>& OutDataArray )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDerivedDataAnimationCompression::Build);

	const double CompressionStartTime = FPlatformTime::Seconds();

	check(DataToCompressPtr.IsValid());
	FCompressibleAnimData& DataToCompress = *DataToCompressPtr.Get();
	FCompressedAnimSequence OutData;

	if (DataToCompress.IsCancelled())
	{
		return false;
	}

	SCOPE_CYCLE_COUNTER(STAT_AnimCompressionDerivedData);
	UE_LOGF(LogAnimationCompression, Log, "Building Anim DDC data for %ls", *DataToCompress.FullName);

	FCompressibleAnimDataResult CompressionResult;

	bool bCompressionSuccessful = false;
	{
		if (DataToCompress.FetchData(DataToCompress.TargetPlatform))
		{
			DataToCompress.Update(OutData);

			const bool bBoneCompressionOk = FAnimationUtils::CompressAnimBones(DataToCompress, CompressionResult);
			if (DataToCompress.IsCancelled())
			{
				return false;
			}
			const bool bCurveCompressionOk = FAnimationUtils::CompressAnimCurves(DataToCompress, OutData);
		
			bCompressionSuccessful = bBoneCompressionOk && bCurveCompressionOk;

#if DO_CHECK
			FString CompressionName = DataToCompress.BoneCompressionSettings->GetFullName();

			ensureMsgf(bCompressionSuccessful, TEXT("Anim Compression failed for Sequence '%s' with compression scheme '%s': compressed data empty"),
												*DataToCompress.FullName,
												*CompressionName);
#endif
		}
		else
		{
			UE_LOGF(LogAnimationCompression, Log, "Failed to fetch FCompressibleAnimData data for %ls", *DataToCompress.FullName);
		}
	}

	bCompressionSuccessful = bCompressionSuccessful && !DataToCompress.IsCancelled();

	if (bCompressionSuccessful)
	{
		OutData.CompressedByteStream = MoveTemp(CompressionResult.CompressedByteStream);
		OutData.CompressedDataStructure = MoveTemp(CompressionResult.AnimData);
		OutData.BoneCompressionCodec = CompressionResult.Codec;

		FMemoryWriter Ar(OutDataArray, true);
		OutData.CompressedRawData = DataToCompress.RawAnimationData;
		OutData.OwnerName = DataToCompress.AnimFName;
		OutData.SerializeCompressedData(Ar, true, nullptr, nullptr, DataToCompress.BoneCompressionSettings, DataToCompress.CurveCompressionSettings); //Save out compressed
	}

	return bCompressionSuccessful;
}

#endif	//WITH_EDITOR
