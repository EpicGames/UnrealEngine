// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if WITH_EDITOR
#include "DerivedDataPluginInterface.h"
#endif

#include "Animation/AnimCompressionTypes.h"

struct FDerivedDataUsageStats;

#if WITH_EDITOR

//////////////////////////////////////////////////////////////////////////
// FDerivedDataAnimationCompression
class FDerivedDataAnimationCompression : public FDerivedDataPluginInterface
{
private:
	// The anim data to compress
	FCompressibleAnimPtr DataToCompressPtr;

	// The Type of anim data to compress (makes up part of DDC key)
	const TCHAR* TypeName;

	// Bulk of asset DDC key
	const FString AssetDDCKey;

public:
	FDerivedDataAnimationCompression(const TCHAR* InTypeName, const FString& InAssetDDCKey);
	virtual ~FDerivedDataAnimationCompression();

	void SetCompressibleData(FCompressibleAnimRef InCompressibleAnimData)
	{
		DataToCompressPtr = InCompressibleAnimData;
	}

	FCompressibleAnimPtr GetCompressibleData() const { return DataToCompressPtr; }

	uint64 GetMemoryUsage() const
	{
		return DataToCompressPtr.IsValid() ? DataToCompressPtr->GetApproxMemoryUsage() : 0;
	}

	virtual const TCHAR* GetPluginName() const override
	{
		return TypeName;
	}

	virtual const TCHAR* GetVersionString() const override;

	virtual FString GetPluginSpecificCacheKeySuffix() const override
	{
		return AssetDDCKey;
	}

	virtual FString GetDebugContextString() const override;

	virtual bool IsBuildThreadsafe() const override
	{
		return true;
	}

	virtual bool Build( TArray<uint8>& OutDataArray) override;

	/** Return true if we can build **/
	bool CanBuild()
	{
		return DataToCompressPtr.IsValid();
	}
};

namespace AnimSequenceCookStats
{
	extern FDerivedDataUsageStats UsageStats;
}

#endif	//WITH_EDITOR
