// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimCurveCompressionCodec.h"
#include "Animation/AnimSequence.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/Package.h"
#include "Interfaces/ITargetPlatform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCurveCompressionCodec)

UAnimCurveCompressionCodec::UAnimCurveCompressionCodec(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimCurveCompressionCodec::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::RemoveAnimCurveCompressionCodecInstanceGuid)
	{
#if !WITH_EDITOR
		if (Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) >= FFortniteReleaseBranchCustomObjectVersion::SerializeAnimCurveCompressionCodecGuidOnCook)
#endif // !WITH_EDITOR
		{
			// We serialized a Guid, read it now and discard it
			check(Ar.IsLoading());
			FGuid InstanceGuid;
			Ar << InstanceGuid;
		}
	}
}

#if WITH_EDITORONLY_DATA
int64 UAnimCurveCompressionCodec::EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const
{
#if WITH_EDITOR
	// This is a conservative estimate that gives a codec enough space to create two raw copies of the input data.
	return 2 * int64(AnimSequence.GetApproxCurveRawSize());
#else
	return -1;
#endif // WITH_EDITOR
}

void UAnimCurveCompressionCodec::PopulateDDCKey(FArchive& Ar)
{
	// We use the UClass name to compute the DDC key to avoid two codecs with equivalent properties (e.g. none)
	// from having the same DDC key. Two codecs with the same values and class name can have the same DDC key
	// since the caller (e.g. anim sequence) factors in raw data and the likes. For a codec class that derives
	// from this, it is their responsibility to factor in compression settings and other inputs into the DDC key.
	FString ClassName = GetClass()->GetName();
	Ar << ClassName;
}
#endif
