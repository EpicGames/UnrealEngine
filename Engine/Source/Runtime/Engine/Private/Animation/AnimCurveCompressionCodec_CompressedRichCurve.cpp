// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimCurveCompressionCodec_CompressedRichCurve.h"
#include "Animation/AnimCompressionTypes.h"
#include "HAL/IConsoleManager.h"
#include "AnimationCompression.h"
#include "Animation/AnimCurveUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCurveCompressionCodec_CompressedRichCurve)

UAnimCurveCompressionCodec_CompressedRichCurve::UAnimCurveCompressionCodec_CompressedRichCurve(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	MaxCurveError = 0.0f;
	UseAnimSequenceSampleRate = true;
	ErrorSampleRate = 60.0f;
#endif
}

// This mirrors in part the FCompressedRichCurve
struct FCurveDesc
{
	TEnumAsByte<ERichCurveCompressionFormat> CompressionFormat;
	TEnumAsByte<ERichCurveKeyTimeCompressionFormat> KeyTimeCompressionFormat;
	TEnumAsByte<ERichCurveExtrapolation> PreInfinityExtrap;
	TEnumAsByte<ERichCurveExtrapolation> PostInfinityExtrap;
	FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys;
	int32 KeyDataOffset;
};

#if WITH_EDITORONLY_DATA

int32 ValidateCompressedRichCurveEvaluation = 0;
static FAutoConsoleVariableRef CVarValidateCompressedRichCurveEvaluation(
	TEXT("a.Compression.ValidateCompressedRichCurveEvaluation"),
	ValidateCompressedRichCurveEvaluation,
	TEXT("1 = runs validation, evaluating the compressed rich curve at animation its sampling rate comparing against the MaxCurveError. 0 = validation disabled"));

bool UAnimCurveCompressionCodec_CompressedRichCurve::Compress(const FCompressibleAnimData& AnimSeq, FAnimCurveCompressionResult& OutResult)
{
	int32 NumCurves = AnimSeq.RawFloatCurves.Num();

	TArray<FCurveDesc> Curves;
	Curves.Reserve(NumCurves);
	Curves.AddUninitialized(NumCurves);

	int32 KeyDataOffset = 0;
	KeyDataOffset += sizeof(FCurveDesc) * NumCurves;

	const FFrameRate& SamplingFrameRate = AnimSeq.SampledFrameRate;
	const double SampleRate = AnimSeq.SampledFrameRate.AsDecimal();

	TArray<uint8> KeyData;

	for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		const FFloatCurve& Curve = AnimSeq.RawFloatCurves[CurveIndex];

		FRichCurve RawCurve = Curve.FloatCurve;	// Copy
		RawCurve.RemoveRedundantAutoTangentKeys(MaxCurveError);

		FCompressedRichCurve CompressedCurve;
		RawCurve.CompressCurve(CompressedCurve, MaxCurveError, SampleRate);

		FCurveDesc& CurveDesc = Curves[CurveIndex];
		CurveDesc.CompressionFormat = CompressedCurve.CompressionFormat;
		CurveDesc.KeyTimeCompressionFormat = CompressedCurve.KeyTimeCompressionFormat;
		CurveDesc.PreInfinityExtrap = CompressedCurve.PreInfinityExtrap;
		CurveDesc.PostInfinityExtrap = CompressedCurve.PostInfinityExtrap;
		CurveDesc.ConstantValueNumKeys = CompressedCurve.ConstantValueNumKeys;
		CurveDesc.KeyDataOffset = KeyDataOffset;

		KeyDataOffset += CompressedCurve.CompressedKeys.Num();
		KeyData.Append(CompressedCurve.CompressedKeys);

		// Compressed curve validation
		if(!!ValidateCompressedRichCurveEvaluation)
		{
			const int32 SamplesToTake = AnimSeq.NumberOfKeys;
			for (int32 KeyIndex = 0; KeyIndex < SamplesToTake; ++KeyIndex)
			{
				// Evaluated RawCurve and Compressed Curve
				const double EvalTime = SamplingFrameRate.AsSeconds(KeyIndex);
				const float RawValue = Curve.FloatCurve.Eval(EvalTime);
				const float CompressedValue = CompressedCurve.Eval(EvalTime);

				const float AbsDelta = FMath::Abs(CompressedValue - RawValue);
				if (!FMath::IsNearlyZero(AbsDelta, MaxCurveError + UE_KINDA_SMALL_NUMBER))
				{
					// Delta larger than tolerated error value
					UE_LOGF(LogAnimationCompression, Warning, "Curve %ls: delta too large %f, between %f and %f, at %f", *Curve.GetName().ToString(), AbsDelta, RawValue, CompressedValue, EvalTime);
				}
			}
		}
	}

	TArray<uint8> TempBytes;
	TempBytes.Reserve(KeyDataOffset);

	// Serialize the compression settings into a temporary array. The archive
	// is flagged as persistent so that machines of different endianness produce
	// identical binary results.
	FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);

	Ar.Serialize(Curves.GetData(), sizeof(FCurveDesc) * NumCurves);
	Ar.Serialize(KeyData.GetData(), KeyData.Num());

	OutResult.CompressedBytes = TempBytes;
	OutResult.Codec = this;

	return true;
}

void UAnimCurveCompressionCodec_CompressedRichCurve::PopulateDDCKey(FArchive& Ar)
{
	Super::PopulateDDCKey(Ar);

	int32 CodecVersion = 0;

	Ar << CodecVersion;
	Ar << MaxCurveError;
	Ar << UseAnimSequenceSampleRate;
	Ar << ErrorSampleRate;
}
#endif

void UAnimCurveCompressionCodec_CompressedRichCurve::DecompressCurves(const FCompressedAnimSequence& AnimSeq, FBlendedCurve& Curves, float CurrentTime) const
{
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();

	if (NumCurves == 0)
	{
		return;
	}

	const uint8* Buffer = AnimSeq.CompressedCurveByteStream.GetData();

	auto GetNameFromIndex = [&IndexedCurveNames](int32 InCurveIndex)
	{
		return IndexedCurveNames[IndexedCurveNames[InCurveIndex].CurveIndex].CurveName;
	};

	// Setup our constant arguments, we'll re-use the structure for every curve
	FCompressedRichCurve::FStaticEvalArgs Args;
	Args.Time = CurrentTime;
	Args.DefaultValue = 0.0f;

	auto GetValueFromIndex = [&IndexedCurveNames, Buffer, &Args](int32 InCurveIndex)
	{
		const FCurveDesc* CurveDescriptions = (const FCurveDesc*)(Buffer);
		const FCurveDesc& Curve = CurveDescriptions[IndexedCurveNames[InCurveIndex].CurveIndex];
		const uint8* CompressedKeys = Buffer + Curve.KeyDataOffset;
		
		Args.CompressionFormat = Curve.CompressionFormat;
		Args.KeyTimeCompressionFormat = Curve.KeyTimeCompressionFormat;
		Args.PreInfinityExtrap = Curve.PreInfinityExtrap;
		Args.PostInfinityExtrap = Curve.PostInfinityExtrap;
		Args.ConstantValueNumKeys = Curve.ConstantValueNumKeys;
		Args.CompressedKeys = CompressedKeys;

		const float Value = FCompressedRichCurve::StaticEval(Args);
		return Value;
	};
	
	UE::Anim::FCurveUtils::BuildSorted(Curves, NumCurves, GetNameFromIndex, GetValueFromIndex, Curves.GetFilter());
}

float UAnimCurveCompressionCodec_CompressedRichCurve::DecompressCurve(const FCompressedAnimSequence& AnimSeq, FName InCurveName, float CurrentTime) const
{
	const uint8* Buffer = AnimSeq.CompressedCurveByteStream.GetData();
	const FCurveDesc* CurveDescriptions = (const FCurveDesc*)(Buffer);
	
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();
	for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		const FName& CurveName = IndexedCurveNames[CurveIndex].CurveName;
		if (CurveName == InCurveName)
		{
			const FCurveDesc& Curve = CurveDescriptions[CurveIndex];
			const uint8* CompressedKeys = Buffer + Curve.KeyDataOffset;

			FCompressedRichCurve::FStaticEvalArgs Args;
			Args.CompressionFormat = Curve.CompressionFormat;
			Args.KeyTimeCompressionFormat = Curve.KeyTimeCompressionFormat;
			Args.PreInfinityExtrap = Curve.PreInfinityExtrap;
			Args.PostInfinityExtrap = Curve.PostInfinityExtrap;
			Args.ConstantValueNumKeys = Curve.ConstantValueNumKeys;
			Args.CompressedKeys = CompressedKeys;
			Args.Time = CurrentTime;
			Args.DefaultValue = 0.0f;

			const float Value = FCompressedRichCurve::StaticEval(Args);
			return Value;
		}
	}

	return 0.0f;
}

