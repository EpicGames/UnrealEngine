// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimCurveCompressionCodec_UniformIndexable.h"
#include "Animation/AnimSequence.h"
#include "EngineLogs.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Animation/AnimCurveUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCurveCompressionCodec_UniformIndexable)

FAnimCurveBufferAccess::FAnimCurveBufferAccess(const UAnimSequenceBase* InSequenceBase, FName InName)
	: NumSamples(INDEX_NONE)
	, SampleRate(0.f)
	, CompressedBuffer(nullptr)
	, RawCurve(nullptr)
	, bUseCompressedData(false)
{
	if (InSequenceBase->HasCurveData(InName, false))
	{
		if (const UAnimSequence* InSequence = Cast<UAnimSequence>(InSequenceBase))
		{
			if (InSequence->IsCurveCompressedDataValid())
			{
				UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = InSequence->GetCompressedData();
				if (UAnimCurveCompressionCodec_UniformIndexable* IndexableCompressedCurve = Cast<UAnimCurveCompressionCodec_UniformIndexable>(CompressedAnimSequence.Get().CurveCompressionCodec))
				{
					bUseCompressedData = IndexableCompressedCurve->GetCurveBufferAndSamples(CompressedAnimSequence.Get(), InName, CompressedBuffer, NumSamples, SampleRate);
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Trying to use FAnimCurveBufferAccess on an animation that is not compressed with UniformIndexable, this is not supported and will not work in cooked builds. Animation: %ls", *InSequence->GetFullName());
				}
			}
		}

		if (!CompressedBuffer)
		{
			RawCurve = (const FFloatCurve*)InSequenceBase->GetCurveData().GetCurveData(InName);

			if (RawCurve)
			{
				NumSamples = RawCurve->FloatCurve.GetNumKeys();
			}
		}
	}
}

float FAnimCurveBufferAccess::GetValue(int32 SampleIndex) const
{
	if (IsValid())
	{
		check(SampleIndex >= 0 && SampleIndex <= NumSamples);

		if (bUseCompressedData)
		{
			return CompressedBuffer[SampleIndex];
		}
		else
		{
			const TArray<FRichCurveKey>& Keys = RawCurve->FloatCurve.GetConstRefOfKeys();
			return Keys[SampleIndex].Value;
		}
	}
	return 0.f;
}

float FAnimCurveBufferAccess::GetTime(int32 SampleIndex) const
{
	if (IsValid())
	{
		check(SampleIndex >= 0 && SampleIndex <= NumSamples);

		if (bUseCompressedData)
		{
			const float InvSampleRate = 1.0f / SampleRate;
			return SampleIndex * InvSampleRate;
		}
		else
		{
			const TArray<FRichCurveKey>& Keys = RawCurve->FloatCurve.GetConstRefOfKeys();
			return Keys[SampleIndex].Time;
		}
	}
	return 0.f;
}

UAnimCurveCompressionCodec_UniformIndexable::UAnimCurveCompressionCodec_UniformIndexable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

#if WITH_EDITORONLY_DATA
bool UAnimCurveCompressionCodec_UniformIndexable::Compress(const FCompressibleAnimData& AnimSeq, FAnimCurveCompressionResult& OutResult)
{
	const int32 NumCurves = AnimSeq.RawFloatCurves.Num();
	const int32 NumSamples = AnimSeq.NumberOfKeys;

	int32 BufferSize = 0;
	BufferSize += sizeof(int32);	// NumSamples
	BufferSize += sizeof(float);	// SampleRate
	BufferSize += sizeof(float) * NumCurves * NumSamples;	// Animated curve samples

	TArray<uint8> Buffer;
	Buffer.Reserve(BufferSize);
	Buffer.AddUninitialized(BufferSize);

	int32 BufferOffset = 0;
	
	int32* NumSamplesPtr = (int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	float* SampleRatePtr = (float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float);

	*NumSamplesPtr = NumSamples;
	*SampleRatePtr = AnimSeq.SampledFrameRate.AsDecimal();

	if (NumCurves > 0 && NumSamples > 0)
	{
		float* AnimatedSamplesPtr = (float*)&Buffer[BufferOffset];
		BufferOffset += sizeof(float) * NumCurves * NumSamples;

		// Write out samples by curve so that we can more easily index curve keys

		// Curve 0 Key 0, Curve 0 Key 1, Curve 0 Key N, Curve 1 Key 0, Curve 1 Key 1, Curve 1 Key N, Curve M Key 0, ...
		for (const FFloatCurve& Curve : AnimSeq.RawFloatCurves)
		{
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				const double FrameSampleTime = AnimSeq.SampledFrameRate.AsSeconds(SampleIndex);
				const float SampleTime = FMath::Clamp(FrameSampleTime, 0.0f, AnimSeq.SequenceLength);
				const float SampleValue = Curve.FloatCurve.Eval(SampleTime);

				(*AnimatedSamplesPtr) = SampleValue;
				++AnimatedSamplesPtr;
			}
		}
	}

	check(BufferOffset == BufferSize);

	OutResult.CompressedBytes = Buffer;
	OutResult.Codec = this;

	return true;
}

int64 UAnimCurveCompressionCodec_UniformIndexable::EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const
{
	const int64 NumCurves = AnimSequence.GetDataModelInterface()->GetFloatCurves().Num();
	const int64 NumSamples = AnimSequence.GetNumberOfSampledKeys();

	int64 BufferSize = 0;
	BufferSize += sizeof(int32);	// NumSamples
	BufferSize += sizeof(float);	// SampleRate
	BufferSize += sizeof(float) * NumCurves * NumSamples;	// Animated curve samples

	return BufferSize;
}

void UAnimCurveCompressionCodec_UniformIndexable::PopulateDDCKey(FArchive& Ar)
{
	Super::PopulateDDCKey(Ar);

	int32 CodecVersion = 0;

	Ar << CodecVersion;
}
#endif

void UAnimCurveCompressionCodec_UniformIndexable::DecompressCurves(const FCompressedAnimSequence& AnimSeq, FBlendedCurve& Curves, float CurrentTime) const
{
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();

	if (NumCurves == 0)
	{
		return;
	}

	const uint8* Buffer = AnimSeq.CompressedCurveByteStream.GetData();

	int32 BufferOffset = 0;

	const int32 NumSamples = *(const int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	if (NumSamples == 0)
	{
		return;
	}

	const float SampleRate = *(const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float);

	const float* AnimatedSamplesPtr = (const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float) * NumCurves * NumSamples;

	const float SamplePoint = CurrentTime * SampleRate;
	const int32 SampleIndex0 = FMath::Clamp(FMath::FloorToInt(SamplePoint), 0, NumSamples - 1);
	const int32 SampleIndex1 = FMath::Min(SampleIndex0 + 1, NumSamples - 1);
	const float InterpolationAlpha = SamplePoint - float(SampleIndex0);

	auto GetNameFromIndex = [&IndexedCurveNames](int32 InCurveIndex)
	{
		return IndexedCurveNames[IndexedCurveNames[InCurveIndex].CurveIndex].CurveName;
	};

	auto GetValueFromIndex = [&IndexedCurveNames, AnimatedSamplesPtr, NumSamples, SampleIndex0, SampleIndex1, InterpolationAlpha](int32 InCurveIndex)
	{
		const int32 CurveIndex = IndexedCurveNames[InCurveIndex].CurveIndex;
		const float* CurveSamplesPtr = AnimatedSamplesPtr + (CurveIndex * NumSamples);

		const float Sample0 = CurveSamplesPtr[SampleIndex0];
		const float Sample1 = CurveSamplesPtr[SampleIndex1];
		const float Sample = FMath::Lerp(Sample0, Sample1, InterpolationAlpha);
		return Sample;
	};

	UE::Anim::FCurveUtils::BuildSorted(Curves, NumCurves, GetNameFromIndex, GetValueFromIndex, Curves.GetFilter());
}

float UAnimCurveCompressionCodec_UniformIndexable::DecompressCurve(const FCompressedAnimSequence& AnimSeq, FName InCurveName, float CurrentTime) const
{
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();

	if (NumCurves == 0)
	{
		return 0.0f;
	}

	const uint8* Buffer = AnimSeq.CompressedCurveByteStream.GetData();

	int32 BufferOffset = 0;

	const int32 NumSamples = *(const int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	if (NumSamples == 0)
	{
		return 0.0f;
	}

	const float SampleRate = *(const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float);

	const float* AnimatedSamplesPtr = (const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float) * NumCurves * NumSamples;

	const float SamplePoint = CurrentTime * SampleRate;
	const int32 SampleIndex0 = FMath::Clamp(FMath::FloorToInt(SamplePoint), 0, NumSamples - 1);
	const int32 SampleIndex1 = FMath::Min(SampleIndex0 + 1, NumSamples - 1);
	const float InterpolationAlpha = SamplePoint - float(SampleIndex0);

	for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		const FName& CurveName = IndexedCurveNames[CurveIndex].CurveName;
		if (CurveName == InCurveName)
		{
			const float* CurveSamplesPtr = AnimatedSamplesPtr + (CurveIndex * NumSamples);

			const float Sample0 = CurveSamplesPtr[SampleIndex0];
			const float Sample1 = CurveSamplesPtr[SampleIndex1];
			const float Sample = FMath::Lerp(Sample0, Sample1, InterpolationAlpha);

			return Sample;
		}
	}

	return 0.0f;
}

bool UAnimCurveCompressionCodec_UniformIndexable::GetCurveBufferAndSamples(const FCompressedAnimSequence& AnimSeq, FName CurveName, const float*& OutCurveBuffer, int32& OutSamples, float& OutSampleRate)
{
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();

	if (NumCurves == 0)
	{
		return false;
	}

	const uint8* Buffer = AnimSeq.CompressedCurveByteStream.GetData();

	int32 BufferOffset = 0;

	OutSamples = *(const int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	if (OutSamples == 0)
	{
		return false;
	}

	OutSampleRate = *(const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float);

	const float* AnimatedSamplesPtr = (const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float) * NumCurves * OutSamples;

	for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		if (IndexedCurveNames[CurveIndex].CurveName == CurveName)
		{
			OutCurveBuffer = AnimatedSamplesPtr + (CurveIndex * OutSamples);
			return true;
		}
	}
	return false;
}

