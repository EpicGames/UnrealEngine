// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimCurveCompressionCodec_UniformlySampled.h"
#include "Animation/AnimCompressionTypes.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCurveCompressionCodec_UniformlySampled)

UAnimCurveCompressionCodec_UniformlySampled::UAnimCurveCompressionCodec_UniformlySampled(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	UseAnimSequenceSampleRate = true;
	SampleRate = 30.0f;
#endif
}

#if WITH_EDITORONLY_DATA
bool UAnimCurveCompressionCodec_UniformlySampled::Compress(const FCompressibleAnimData& AnimSeq, FAnimCurveCompressionResult& OutResult)
{
	const int32 NumCurves = AnimSeq.RawFloatCurves.Num();
	const float Duration = AnimSeq.SequenceLength;

	int32 NumSamples;
	float SampleRate_;
	if (UseAnimSequenceSampleRate)
	{
		NumSamples = AnimSeq.NumberOfKeys;
		SampleRate_ = AnimSeq.SampledFrameRate.AsDecimal();
	}
	else
	{
		// If our duration isn't an exact multiple of the sample rate, we'll round
		// and end up with a sample rate slightly corrected to make sure we spread
		// the resulting error over the whole duration
		NumSamples = FMath::RoundToInt(Duration * SampleRate) + 1;
		SampleRate_ = (NumSamples - 1) / Duration;
	}

	int32 NumConstantCurves = 0;
	for (const FFloatCurve& Curve : AnimSeq.RawFloatCurves)
	{
		if (Curve.FloatCurve.IsConstant())
		{
			NumConstantCurves++;
		}
	}

	const int32 NumAnimatedCurves = NumCurves - NumConstantCurves;

	// 1 bit per curve, round up
	const int32 ConstantBitsetSize = sizeof(uint32) * ((NumCurves + 31) / 32);

	int32 BufferSize = 0;
	BufferSize += sizeof(int32);	// NumConstantCurves
	BufferSize += sizeof(int32);	// NumSamples
	BufferSize += sizeof(float);	// SampleRate
	BufferSize += ConstantBitsetSize;								// Constant curve bitset
	BufferSize += sizeof(float) * NumConstantCurves;				// Constant curve samples
	BufferSize += sizeof(float) * NumAnimatedCurves * NumSamples;	// Animated curve samples

	TArray<uint8> Buffer;
	Buffer.Reserve(BufferSize);
	Buffer.AddUninitialized(BufferSize);

	int32 BufferOffset = 0;
	int32* NumConstantCurvesPtr = (int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	int32* NumSamplesPtr = (int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	float* SampleRatePtr = (float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float);

	*NumConstantCurvesPtr = NumConstantCurves;
	*NumSamplesPtr = NumSamples;
	*SampleRatePtr = SampleRate_;

	if (NumCurves > 0 && NumSamples > 0)
	{
		uint32* ConstantCurvesBitsetPtr = (uint32*)&Buffer[BufferOffset];
		BufferOffset += ConstantBitsetSize;
		FMemory::Memzero(ConstantCurvesBitsetPtr, ConstantBitsetSize);

		float* ConstantSamplesPtr = NumConstantCurves > 0 ? (float*)&Buffer[BufferOffset] : nullptr;
		BufferOffset += sizeof(float) * NumConstantCurves;

		float* AnimatedSamplesPtr = NumAnimatedCurves > 0 ? (float*)&Buffer[BufferOffset] : nullptr;
		BufferOffset += sizeof(float) * NumAnimatedCurves * NumSamples;

		if(ConstantSamplesPtr)
		{
			for (int32 CurveIndex = 0, ConstantCurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				const FFloatCurve& Curve = AnimSeq.RawFloatCurves[CurveIndex];
				if (Curve.FloatCurve.IsConstant())
				{
					if (Curve.FloatCurve.IsEmpty())
					{
						ConstantSamplesPtr[ConstantCurveIndex] = Curve.FloatCurve.DefaultValue;
					}
					else
					{
						ConstantSamplesPtr[ConstantCurveIndex] = Curve.FloatCurve.Keys[0].Value;
					}

					// Bitset uses little-endian bit ordering
					ConstantCurvesBitsetPtr[CurveIndex / 32] |= 1 << (CurveIndex % 32);
					ConstantCurveIndex++;
				}
			}
		}

		if(AnimatedSamplesPtr)
		{
			// Write out samples sorted by time first in order to have everything contiguous in memory
			// for improved cache locality
			// Curve 0 Key 0, Curve 0 Key 1, Curve 0 Key N, Curve 1 Key 0, Curve 1 Key 1, Curve 1 Key N, Curve M Key 0, ...
			const float InvSampleRate = 1.0f / SampleRate_;
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				const float SampleTime = FMath::Clamp(SampleIndex * InvSampleRate, 0.0f, Duration);
				float* AnimatedSamples = AnimatedSamplesPtr + (SampleIndex * NumAnimatedCurves);

				for (int32 CurveIndex = 0, AnimatedCurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
				{
					const FFloatCurve& Curve = AnimSeq.RawFloatCurves[CurveIndex];
					if (Curve.FloatCurve.IsConstant())
					{
						// Skip constant curves, their data has already been written
						continue;
					}

					const float SampleValue = Curve.FloatCurve.Eval(SampleTime);

					AnimatedSamples[AnimatedCurveIndex] = SampleValue;
					AnimatedCurveIndex++;
				}
			}
		}
	}

	check(BufferOffset == BufferSize);

	OutResult.CompressedBytes = Buffer;
	OutResult.Codec = this;

	return true;
}

int64 UAnimCurveCompressionCodec_UniformlySampled::EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const
{
	// Same calculation as above
	int64 NumSamples;
	if (UseAnimSequenceSampleRate)
	{
		NumSamples = AnimSequence.GetNumberOfSampledKeys();
	}
	else
	{
		NumSamples = FMath::RoundToInt(AnimSequence.GetPlayLength() * SampleRate) + 1;
	}

	const int64 NumCurves = AnimSequence.GetDataModelInterface()->GetFloatCurves().Num();

	const int64 ConstantBitsetSize = sizeof(uint32) * ((NumCurves + 31) / 32);

	// Assume every curve is animated since they use more memory
	const int64 NumConstantCurves = 0;
	const int64 NumAnimatedCurves = NumCurves;

	int64 BufferSize = 0;
	BufferSize += sizeof(int32);	// NumConstantCurves
	BufferSize += sizeof(int32);	// NumSamples
	BufferSize += sizeof(float);	// SampleRate
	BufferSize += ConstantBitsetSize;								// Constant curve bitset
	BufferSize += sizeof(float) * NumConstantCurves;				// Constant curve samples
	BufferSize += sizeof(float) * NumAnimatedCurves * NumSamples;	// Animated curve samples

	return BufferSize;
}

void UAnimCurveCompressionCodec_UniformlySampled::PopulateDDCKey(FArchive& Ar)
{
	Super::PopulateDDCKey(Ar);

	int32 CodecVersion = 0;

	Ar << CodecVersion;
	Ar << UseAnimSequenceSampleRate;
	Ar << SampleRate;
}
#endif

void UAnimCurveCompressionCodec_UniformlySampled::DecompressCurves(const FCompressedAnimSequence& AnimSeq, FBlendedCurve& Curves, float CurrentTime) const
{
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();

	if (NumCurves == 0)
	{
		return;
	}

	const uint8* Buffer = AnimSeq.CompressedCurveByteStream.GetData();

	int32 BufferOffset = 0;
	const int32 NumConstantCurves = *(const int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	const int32 NumAnimatedCurves = NumCurves - NumConstantCurves;

	const int32 NumSamples = *(const int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	if (NumSamples == 0)
	{
		return;
	}

	const float SampleRate_ = *(const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float);

	const uint32* ConstantCurvesBitsetPtr = (const uint32*)&Buffer[BufferOffset];
	const int32 ConstantBitsetSize = sizeof(uint32) * ((NumCurves + 31) / 32);
	BufferOffset += ConstantBitsetSize;

	const float* ConstantSamplesPtr = (const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float) * NumConstantCurves;

	const float* AnimatedSamplesPtr = (const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float) * NumAnimatedCurves * NumSamples;

	const float SamplePoint = CurrentTime * SampleRate_;
	const int32 SampleIndex0 = FMath::Clamp(FMath::FloorToInt(SamplePoint), 0, NumSamples - 1);
	const int32 SampleIndex1 = FMath::Min(SampleIndex0 + 1, NumSamples - 1);
	const float InterpolationAlpha = SamplePoint - float(SampleIndex0);

	const float* AnimatedSamples0 = AnimatedSamplesPtr + (SampleIndex0 * NumAnimatedCurves);
	const float* AnimatedSamples1 = AnimatedSamplesPtr + (SampleIndex1 * NumAnimatedCurves);

	int32 ConstantCurveIndex = 0, AnimatedCurveIndex = 0;

	auto GetNameFromIndex = [&IndexedCurveNames](int32 InCurveIndex)
	{
		return IndexedCurveNames[IndexedCurveNames[InCurveIndex].CurveIndex].CurveName;
	};

	auto GetValueFromIndex = [&IndexedCurveNames, ConstantCurvesBitsetPtr, &ConstantCurveIndex, &AnimatedCurveIndex, ConstantSamplesPtr, AnimatedSamples0, AnimatedSamples1, InterpolationAlpha](int32 InCurveIndex) 
	{
		const int32 CurveIndex = IndexedCurveNames[InCurveIndex].CurveIndex;
		const bool bIsConstant = (ConstantCurvesBitsetPtr[CurveIndex / 32] & (1 << (CurveIndex % 32))) != 0;

		float Sample;
		if (bIsConstant)
		{
			Sample = ConstantSamplesPtr[ConstantCurveIndex];
		}
		else
		{
			const float Sample0 = AnimatedSamples0[AnimatedCurveIndex];
			const float Sample1 = AnimatedSamples1[AnimatedCurveIndex];
			Sample = FMath::Lerp(Sample0, Sample1, InterpolationAlpha);
		}

		(bIsConstant ? ConstantCurveIndex : AnimatedCurveIndex)++;

		return Sample;
	};

	UE::Anim::FCurveUtils::BuildSorted(Curves, NumCurves, GetNameFromIndex, GetValueFromIndex, Curves.GetFilter());
}

float UAnimCurveCompressionCodec_UniformlySampled::DecompressCurve(const FCompressedAnimSequence& AnimSeq, FName InCurveName, float CurrentTime) const
{
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();

	if (NumCurves == 0)
	{
		return 0.0f;
	}

	const uint8* Buffer = AnimSeq.CompressedCurveByteStream.GetData();

	int32 BufferOffset = 0;
	const int32 NumConstantCurves = *(const int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	const int32 NumSamples = *(const int32*)&Buffer[BufferOffset];
	BufferOffset += sizeof(int32);

	const float SampleRate_ = *(const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float);

	const uint32* ConstantCurvesBitsetPtr = (const uint32*)&Buffer[BufferOffset];
	const int32 ConstantBitsetSize = sizeof(uint32) * ((NumCurves + 31) / 32);
	BufferOffset += ConstantBitsetSize;
	
	const float* ConstantSamplesPtr = (const float*)&Buffer[BufferOffset];
	BufferOffset += sizeof(float) * NumConstantCurves;

	const float* AnimatedSamplesPtr = (const float*)&Buffer[BufferOffset];
	const int32 NumAnimatedCurves = NumCurves - NumConstantCurves;
	BufferOffset += sizeof(float) * NumAnimatedCurves * NumSamples;

	for (int32 CurveIndex = 0, ConstantCurveIndex = 0, AnimatedCurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		const FName& CurveName = IndexedCurveNames[CurveIndex].CurveName;
		const bool bIsConstant = (ConstantCurvesBitsetPtr[CurveIndex / 32] & (1 << (CurveIndex % 32))) != 0;
		if (CurveName == InCurveName)
		{
			float Sample;
			if (bIsConstant)
			{
				Sample = ConstantSamplesPtr[ConstantCurveIndex];
			}
			else
			{
				const float SamplePoint = CurrentTime * SampleRate_;
				const int32 SampleIndex0 = FMath::Clamp(FMath::FloorToInt(SamplePoint), 0, NumSamples - 1);
				const int32 SampleIndex1 = FMath::Min(SampleIndex0 + 1, NumSamples - 1);
				const float InterpolationAlpha = SamplePoint - float(SampleIndex0);

				const float Sample0 = AnimatedSamplesPtr[(SampleIndex0 * NumAnimatedCurves) + CurveIndex];
				const float Sample1 = AnimatedSamplesPtr[(SampleIndex1 * NumAnimatedCurves) + CurveIndex];
				Sample = FMath::Lerp(Sample0, Sample1, InterpolationAlpha);
			}

			return Sample;
		}

		(bIsConstant ? ConstantCurveIndex : AnimatedCurveIndex)++;
	}

	return 0.0f;
}

