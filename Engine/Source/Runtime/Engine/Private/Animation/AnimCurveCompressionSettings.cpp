// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimCurveCompressionSettings.h"
#include "Animation/AnimCompressionTypes.h"
#include "Animation/AnimCurveCompressionCodec.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCurveCompressionSettings)

UAnimCurveCompressionSettings::UAnimCurveCompressionSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAnimCurveCompressionCodec* UAnimCurveCompressionSettings::GetCodec(const FString& Path)
{
	return Codec->GetCodec(Path);
}

#if WITH_EDITORONLY_DATA
void UAnimCurveCompressionSettings::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	if (Codec != nullptr)
	{
		OutDeps.Add(Codec);
	}
}

bool UAnimCurveCompressionSettings::AreSettingsValid() const
{
	return Codec != nullptr && Codec->IsCodecValid();
}

bool UAnimCurveCompressionSettings::Compress(const FCompressibleAnimData& AnimSeq, FCompressedAnimSequence& OutCompressedData) const
{
	if (Codec == nullptr || !AreSettingsValid())
	{
		return false;
	}

	FAnimCurveCompressionResult CompressionResult;
	bool Success = Codec->Compress(AnimSeq, CompressionResult);
	if (Success)
	{
		OutCompressedData.CompressedCurveByteStream = CompressionResult.CompressedBytes;
		OutCompressedData.CurveCompressionCodec = CompressionResult.Codec;
	}

	return Success;
}

void UAnimCurveCompressionSettings::PopulateDDCKey(FArchive& Ar)
{
	if (Codec)
	{
		Codec->PopulateDDCKey(Ar);
	}
	else
	{
		static FString NoCodecString("<Missing Codec>");
		Ar << NoCodecString;
	}
}

#endif

