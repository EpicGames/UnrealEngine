// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimCurveMetadata.h"
#include "Animation/Skeleton.h"
#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "AnimCurveMetaData" 

FArchive& operator<<(FArchive& Ar, FCurveMetaData& B)
{
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);

	Ar << B.Type.bMaterial;
	Ar << B.Type.bMorphtarget;
	Ar << B.LinkedBones;

	if (Ar.CustomVer(FAnimPhysObjectVersion::GUID) >= FAnimPhysObjectVersion::AddLODToCurveMetaData)
	{
		Ar << B.MaxLOD;
	}

	return Ar;
}

bool FCurveMetaData::Serialize(FArchive& Ar)
{
	Ar << *this;
	return true;
}

FCurveMetaData::FCurveMetaData()
	: MaxLOD(0xFF)
{
}

#if WITH_EDITOR

void UAnimCurveMetaData::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UAnimCurveMetaData::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	// Add curve metadata IDs to a tag list, or a delimiter if we have no curve metadata.
	// The delimiter is necessary so we can distinguish between data with no curves and old data, as the asset registry
	// strips tags that have empty values 
	TStringBuilder<256> CurvesBuilder;
	CurvesBuilder << USkeleton::CurveTagDelimiter;

	ForEachCurveMetaData([&CurvesBuilder](FName InCurveName, const FCurveMetaData&)
	{
		CurvesBuilder << InCurveName;
		CurvesBuilder << USkeleton::CurveTagDelimiter;
	});

	Context.AddTag(FAssetRegistryTag(USkeleton::CurveNameTag, CurvesBuilder.ToString(), FAssetRegistryTag::TT_Hidden));
}

#endif //WITH_EDITOR

void UAnimCurveMetaData::ForEachCurveMetaData(const TFunctionRef<void(FName, const FCurveMetaData&)>& InFunction) const
{
	for(const TPair<FName, FCurveMetaData>& MetaDataPair : CurveMetaData)
	{
		InFunction(MetaDataPair.Key, MetaDataPair.Value);
	}
}

const FCurveMetaData* UAnimCurveMetaData::GetCurveMetaData(FName InCurveName) const
{
	return CurveMetaData.Find(InCurveName);
}

bool UAnimCurveMetaData::AddCurveMetaData(FName InCurveName, bool bInTransact)
{
	if(!CurveMetaData.Contains(InCurveName))
	{
#if WITH_EDITOR
		if (bInTransact)
		{
			GEditor->BeginTransaction(LOCTEXT("AddCurveMetaData", "Add Curve Metadata"));
		}
		Modify();
#endif

		CurveMetaData.Add(InCurveName);

		IncreaseVersionNumber();
		OnCurveMetaDataChanged.Broadcast();

#if WITH_EDITOR
		if (bInTransact)
		{
			GEditor->EndTransaction();
		}
#endif
		return true;
	}
	return false;
}

FCurveMetaData* UAnimCurveMetaData::GetCurveMetaData(FName InCurveName)
{
	return CurveMetaData.Find(InCurveName);
}

void UAnimCurveMetaData::GetCurveMetaDataNames(TArray<FName>& OutNames) const
{
	CurveMetaData.GenerateKeyArray(OutNames);
}

void UAnimCurveMetaData::RefreshBoneIndices(USkeleton* InSkeleton)
{
	// initialize bone indices for skeleton
	for (TPair<FName, FCurveMetaData>& CurveMetaDataPair : CurveMetaData)
	{
		for (FBoneReference& LinkedBone : CurveMetaDataPair.Value.LinkedBones)
		{
			LinkedBone.Initialize(InSkeleton);
		}
	}

	IncreaseVersionNumber();
}

uint16 UAnimCurveMetaData::GetVersionNumber() const
{
	return VersionNumber;
}

#if WITH_EDITOR

bool UAnimCurveMetaData::RenameCurveMetaData(FName OldName, FName NewName)
{
	if(OldName != NewName)
	{
		// Dont allow renaming on top of an existing entry
		if(CurveMetaData.Contains(NewName))
		{
			return false;
		}

		FScopedTransaction Transaction(LOCTEXT("RenameCurveMetaData", "Rename Curve Metadata"));
		Modify();

		// Remove & re-add
		FCurveMetaData ExistingCopy;
		if(CurveMetaData.RemoveAndCopyValue(OldName, ExistingCopy))
		{
			CurveMetaData.Add(NewName, ExistingCopy);

			IncreaseVersionNumber();
			OnCurveMetaDataChanged.Broadcast();
			return true;
		}
	}

	return false;
}

bool UAnimCurveMetaData::RemoveCurveMetaData(FName CurveName)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveCurveMetaData", "Remove Curve Metadata"));
	Modify();

	bool bChanged = CurveMetaData.Remove(CurveName) > 0;
	if(bChanged)
	{
		IncreaseVersionNumber();
		OnCurveMetaDataChanged.Broadcast();
	}
	return bChanged;
}

bool UAnimCurveMetaData::RemoveCurveMetaData(TArrayView<FName> CurveNames)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveCurveMetaData", "Remove Curve Metadata"));
	Modify();

	bool bChanged = false;
	for(const FName& CurveName : CurveNames)
	{
		bChanged |= RemoveCurveMetaData(CurveName);
	}
	if(bChanged)
	{
		IncreaseVersionNumber();
		OnCurveMetaDataChanged.Broadcast();
	}
	return bChanged;
}


bool UAnimCurveMetaData::GetCurveMetaDataMaterial(FName CurveName) const
{
	const FCurveMetaData* MetaData = GetCurveMetaData(CurveName);
	if (MetaData)
	{
		return MetaData->Type.bMaterial;
	}

	return false;
}

bool UAnimCurveMetaData::GetCurveMetaDataMorphTarget(FName CurveName) const
{
	const FCurveMetaData* MetaData = GetCurveMetaData(CurveName);
	if (MetaData)
	{
		return MetaData->Type.bMorphtarget;
	}

	return false;
}

void UAnimCurveMetaData::SetCurveMetaDataMaterial(FName CurveName, bool bOverrideMaterial)
{
	FCurveMetaData* MetaData = GetCurveMetaData(CurveName);
	if (MetaData)
	{
		FScopedTransaction Transaction(LOCTEXT("SetCurveMaterialFlag", "Set Curve Material Flag"));
		Modify();

		// override curve data
		MetaData->Type.bMaterial = bOverrideMaterial;

		IncreaseVersionNumber();
		OnCurveMetaDataChanged.Broadcast();
	}
}

void UAnimCurveMetaData::SetCurveMetaDataMorphTarget(FName CurveName, bool bOverrideMorphTarget)
{
	FCurveMetaData* MetaData = GetCurveMetaData(CurveName);
	if (MetaData)
	{
		FScopedTransaction Transaction(LOCTEXT("SetCurveMoprhTargetFlag", "Set Curve Morph Target Flag"));
		Modify();
		
		// override curve data
		MetaData->Type.bMorphtarget = bOverrideMorphTarget;

		IncreaseVersionNumber();
		OnCurveMetaDataChanged.Broadcast();
	}
}

void UAnimCurveMetaData::SetCurveMetaDataBoneLinks(FName CurveName, const TArray<FBoneReference>& BoneLinks, uint8 InMaxLOD, USkeleton* InSkeleton)
{
	FCurveMetaData* MetaData = GetCurveMetaData(CurveName);
	if (MetaData)
	{
		FScopedTransaction Transaction(LOCTEXT("SetCurveBoneLinksFlag", "Set Curve Bone Links"));
		Modify();

		// override curve data
		MetaData->LinkedBones = BoneLinks;
		MetaData->MaxLOD = InMaxLOD;

		if(InSkeleton)
		{
			for (FBoneReference& BoneReference : MetaData->LinkedBones)
			{
				BoneReference.Initialize(InSkeleton);
			}
		}

		IncreaseVersionNumber();
		OnCurveMetaDataChanged.Broadcast();
	}
}

FDelegateHandle UAnimCurveMetaData::RegisterOnCurveMetaDataChanged(const FSimpleMulticastDelegate::FDelegate& InOnCurveMetaDataChanged)
{
	return OnCurveMetaDataChanged.Add(InOnCurveMetaDataChanged);
}

void UAnimCurveMetaData::UnregisterOnCurveMetaDataChanged(FDelegateHandle InHandle)
{
	OnCurveMetaDataChanged.Remove(InHandle);
}

#endif // WITH_EDITOR

void UAnimCurveMetaData::IncreaseVersionNumber()
{
	VersionNumber++;

	// Zero is 'invalid' for cached values elsewhere, so skip it here
	if(VersionNumber == 0)
	{
		VersionNumber++;
	}
}

#undef LOCTEXT_NAMESPACE