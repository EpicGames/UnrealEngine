// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "Animation/Skeleton.h"
#include "Stats/Stats.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/AnimObjectVersion.h"
#include "Math/RandomStream.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/Skeleton.h"
#include "BoneContainer.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Animation/AnimCurveUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCurveTypes)

DECLARE_CYCLE_STAT(TEXT("EvalRawCurveData"), STAT_EvalRawCurveData, STATGROUP_Anim);

namespace UE::Anim
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<float> FBaseBlendedCurve_DEPRECATED::CurveWeights;
	TBitArray<> FBaseBlendedCurve_DEPRECATED::ValidCurveWeights;
	TArray<uint16> const* FBaseBlendedCurve_DEPRECATED::UIDToArrayIndexLUT;
	uint16 FBaseBlendedCurve_DEPRECATED::NumValidCurveCount;
	bool FBaseBlendedCurve_DEPRECATED::bInitialized;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

/////////////////////////////////////////////////////
// FFloatCurve

void FAnimCurveBase::PostSerializeFixup(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::SmartNameRefactor)
	{
		if (Ar.UEVer() >= VER_UE4_SKELETON_ADD_SMARTNAMES)
		{
			SmartName::UID_Type CurveUid = SmartName::MaxUID;
			Ar << CurveUid;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Name_DEPRECATED.UID = CurveUid;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif
}

bool FAnimCurveBase::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	// Return false to defer to regular serialization 
	return false;
}

void FAnimCurveBase::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::SmartNameRefactor)
		{
			// Between CLs 3002109 (SmartNameRefactor) and 3026802 (PoseAssetSupportPerBoneMask), PoseAssets had no custom version serialized into their archive, so we
			// cant properly upgrade curves stored there. We instead assume that if LastObservedName_DEPRECATED is not
			// NAME_None, we can use it
			if(LastObservedName_DEPRECATED != NAME_None)
			{
				Name_DEPRECATED.DisplayName = LastObservedName_DEPRECATED;
			}
		}

		if(Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::AnimSequenceCurveColors)
		{
			// Need to set the curve name before we generate a new color
			CurveName = Name_DEPRECATED.DisplayName;
			Color = MakeColor(CurveName);
		}

		if(Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimationRemoveSmartNames)
		{
			CurveName = Name_DEPRECATED.DisplayName;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
}

void FAnimCurveBase::SetCurveTypeFlag(EAnimAssetCurveFlags InFlag, bool bValue)
{
	if (bValue)
	{
		CurveTypeFlags |= InFlag;
	}
	else
	{
		CurveTypeFlags &= ~InFlag;
	}
}

void FAnimCurveBase::ToggleCurveTypeFlag(EAnimAssetCurveFlags InFlag)
{
	bool Current = GetCurveTypeFlag(InFlag);
	SetCurveTypeFlag(InFlag, !Current);
}

bool FAnimCurveBase::GetCurveTypeFlag(EAnimAssetCurveFlags InFlag) const
{
	return (CurveTypeFlags & InFlag) != 0;
}


void FAnimCurveBase::SetCurveTypeFlags(int32 NewCurveTypeFlags)
{
	CurveTypeFlags = NewCurveTypeFlags;
}

int32 FAnimCurveBase::GetCurveTypeFlags() const
{
	return CurveTypeFlags;
}

#if WITH_EDITORONLY_DATA
FLinearColor FAnimCurveBase::MakeColor(const FName& CurveName)
{
	// Create a color based on the hash of the name
	FRandomStream Stream(GetTypeHash(CurveName));
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	return FLinearColor::MakeFromHSV8(Hue, 196, 196);
}
#endif

////////////////////////////////////////////////////
//  FFloatCurve

// we don't want to have = operator. This only copies curves, but leaving naming and everything else intact. 
void FFloatCurve::CopyCurve(const FFloatCurve& SourceCurve)
{
	FloatCurve = SourceCurve.FloatCurve;
}

float FFloatCurve::Evaluate(float CurrentTime) const
{
	return FloatCurve.Eval(CurrentTime);
}

void FFloatCurve::UpdateOrAddKey(float NewKey, float CurrentTime)
{
	FloatCurve.UpdateOrAddKey(CurrentTime, NewKey);
}

void FFloatCurve::GetKeys(TArray<float>& OutTimes, TArray<float>& OutValues) const
{
	const int32 NumKeys = FloatCurve.GetNumKeys();
	OutTimes.Empty(NumKeys);
	OutValues.Empty(NumKeys);
	for (auto It = FloatCurve.GetKeyHandleIterator(); It; ++It)
	{
		const FKeyHandle KeyHandle = *It;
		const float KeyTime = FloatCurve.GetKeyTime(KeyHandle);
		const float Value = FloatCurve.Eval(KeyTime);

		OutTimes.Add(KeyTime);
		OutValues.Add(Value);
	}
}

void FFloatCurve::Resize(float NewLength, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime)
{
	FloatCurve.ReadjustTimeRange(0, NewLength, bInsert, OldStartTime, OldEndTime);
}
////////////////////////////////////////////////////
//  FVectorCurve

// we don't want to have = operator. This only copies curves, but leaving naming and everything else intact. 
void FVectorCurve::CopyCurve(const FVectorCurve& SourceCurve)
{
	FloatCurves[0] = SourceCurve.FloatCurves[0];
	FloatCurves[1] = SourceCurve.FloatCurves[1];
	FloatCurves[2] = SourceCurve.FloatCurves[2];
}

FVector FVectorCurve::Evaluate(float CurrentTime, float BlendWeight) const
{
	FVector Value;

	Value.X = FloatCurves[(int32)EIndex::X].Eval(CurrentTime)*BlendWeight;
	Value.Y = FloatCurves[(int32)EIndex::Y].Eval(CurrentTime)*BlendWeight;
	Value.Z = FloatCurves[(int32)EIndex::Z].Eval(CurrentTime)*BlendWeight;

	return Value;
}

void FVectorCurve::UpdateOrAddKey(const FVector& NewKey, float CurrentTime)
{
	FloatCurves[(int32)EIndex::X].UpdateOrAddKey(CurrentTime, NewKey.X);
	FloatCurves[(int32)EIndex::Y].UpdateOrAddKey(CurrentTime, NewKey.Y);
	FloatCurves[(int32)EIndex::Z].UpdateOrAddKey(CurrentTime, NewKey.Z);
}

void FVectorCurve::GetKeys(TArray<float>& OutTimes, TArray<FVector>& OutValues) const
{
	// Determine curve with most keys
	int32 MaxNumKeys = 0;
	int32 UsedCurveIndex = INDEX_NONE;
	for (int32 CurveIndex = 0; CurveIndex < 3; ++CurveIndex)
	{
		const int32 NumKeys = FloatCurves[CurveIndex].GetNumKeys();
		if (NumKeys > MaxNumKeys)
		{
			MaxNumKeys = NumKeys;
			UsedCurveIndex = CurveIndex;
		}
	}

	if (UsedCurveIndex != INDEX_NONE)
	{
		OutTimes.Empty(MaxNumKeys);
		OutValues.Empty(MaxNumKeys);
		for (auto It = FloatCurves[UsedCurveIndex].GetKeyHandleIterator(); It; ++It)
		{
			const FKeyHandle KeyHandle = *It;
			const float KeyTime = FloatCurves[UsedCurveIndex].GetKeyTime(KeyHandle);
			const FVector Value = Evaluate(KeyTime, 1.0f);

			OutTimes.Add(KeyTime);
			OutValues.Add(Value);
		}
	}
}

void FVectorCurve::Resize(float NewLength, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime)
{
	FloatCurves[(int32)EIndex::X].ReadjustTimeRange(0, NewLength, bInsert, OldStartTime, OldEndTime);
	FloatCurves[(int32)EIndex::Y].ReadjustTimeRange(0, NewLength, bInsert, OldStartTime, OldEndTime);
	FloatCurves[(int32)EIndex::Z].ReadjustTimeRange(0, NewLength, bInsert, OldStartTime, OldEndTime);
}

int32 FVectorCurve::GetNumKeys() const
{
	int32 MaxNumKeys = 0;
	for (int32 CurveIndex = 0; CurveIndex < 3; ++CurveIndex)
	{
		const int32 NumKeys = FloatCurves[CurveIndex].GetNumKeys();
		MaxNumKeys = FMath::Max(MaxNumKeys, NumKeys);
	}

	return MaxNumKeys;
}

////////////////////////////////////////////////////
//  FTransformCurve

// we don't want to have = operator. This only copies curves, but leaving naming and everything else intact. 
void FTransformCurve::CopyCurve(const FTransformCurve& SourceCurve)
{
	TranslationCurve.CopyCurve(SourceCurve.TranslationCurve);
	RotationCurve.CopyCurve(SourceCurve.RotationCurve);
	ScaleCurve.CopyCurve(SourceCurve.ScaleCurve);
}

FTransform FTransformCurve::Evaluate(float CurrentTime, float BlendWeight) const
{
	FTransform Value;
	Value.SetTranslation(TranslationCurve.Evaluate(CurrentTime, BlendWeight));
	if (ScaleCurve.DoesContainKey())
	{
		Value.SetScale3D(ScaleCurve.Evaluate(CurrentTime, BlendWeight));
	}
	else
	{
		Value.SetScale3D(FVector(1.f));
	}

	// blend rotation float curve
	FVector RotationAsVector = RotationCurve.Evaluate(CurrentTime, BlendWeight);
	// pitch, yaw, roll order - please check AddKey function
	FRotator Rotator(RotationAsVector.Y, RotationAsVector.Z, RotationAsVector.X);
	Value.SetRotation(FQuat(Rotator));

	return Value;
}

void FTransformCurve::UpdateOrAddKey(const FTransform& NewKey, float CurrentTime)
{
	TranslationCurve.UpdateOrAddKey(NewKey.GetTranslation(), CurrentTime);
	// pitch, yaw, roll order - please check Evaluate function
	FVector RotationAsVector;
	FRotator Rotator = NewKey.GetRotation().Rotator();
	RotationAsVector.X = Rotator.Roll;
	RotationAsVector.Y = Rotator.Pitch;
	RotationAsVector.Z = Rotator.Yaw;

	RotationCurve.UpdateOrAddKey(RotationAsVector, CurrentTime);
	ScaleCurve.UpdateOrAddKey(NewKey.GetScale3D(), CurrentTime);
}

void FTransformCurve::GetKeys(TArray<float>& OutTimes, TArray<FTransform>& OutValues) const
{
	const FVectorCurve* UsedCurve = nullptr;
	int32 MaxNumKeys = 0;

	int32 NumKeys = TranslationCurve.GetNumKeys();
	if (NumKeys > MaxNumKeys)
	{
		UsedCurve = &TranslationCurve;
		MaxNumKeys = NumKeys;
	}

	NumKeys = RotationCurve.GetNumKeys();
	if (NumKeys > MaxNumKeys)
	{
		UsedCurve = &RotationCurve;
		MaxNumKeys = NumKeys;
	}

	NumKeys = ScaleCurve.GetNumKeys();
	if (NumKeys > MaxNumKeys)
	{
		UsedCurve = &ScaleCurve;
		MaxNumKeys = NumKeys;
	}

	if (UsedCurve != nullptr)
	{
		int32 UsedChannelIndex = 0;
		for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
		{
			if (UsedCurve->FloatCurves[ChannelIndex].GetNumKeys() == MaxNumKeys)
			{
				UsedChannelIndex = ChannelIndex;
				break;
			}
		}

		OutTimes.Empty(MaxNumKeys);
		OutValues.Empty(MaxNumKeys);
		for (auto It = UsedCurve->FloatCurves[UsedChannelIndex].GetKeyHandleIterator(); It; ++It)
		{
			const FKeyHandle KeyHandle = *It;
			const float KeyTime = UsedCurve->FloatCurves[UsedChannelIndex].GetKeyTime(KeyHandle);
			const FTransform Value = Evaluate(KeyTime, 1.0f);

			OutTimes.Add(KeyTime);
			OutValues.Add(Value);
		}
	}
}

void FTransformCurve::Resize(float NewLength, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime)
{
	TranslationCurve.Resize(NewLength, bInsert, OldStartTime, OldEndTime);
	RotationCurve.Resize(NewLength, bInsert, OldStartTime, OldEndTime);
	ScaleCurve.Resize(NewLength, bInsert, OldStartTime, OldEndTime);
}

const FVectorCurve* FTransformCurve::GetVectorCurveByIndex(int32 Index) const
{
	const FVectorCurve* Curve = nullptr;

	if (Index == 0)
	{
		Curve = &TranslationCurve;
	}
	else if (Index == 1)
	{
		Curve = &RotationCurve;
	}
	else if (Index == 2)
	{
		Curve = &ScaleCurve;
	}

	return Curve;
}

FVectorCurve* FTransformCurve::GetVectorCurveByIndex(int32 Index)
{
	FVectorCurve* Curve = nullptr;

	if (Index == 0)
	{
		Curve = &TranslationCurve;
	}
	else if (Index == 1)
	{
		Curve = &RotationCurve;
	}
	else if (Index == 2)
	{
		Curve = &ScaleCurve;
	}

	return Curve;
}

////////////////////////////////////////////////////
//  FCachedFloatCurve

bool FCachedFloatCurve::IsValid(const UAnimSequenceBase* InAnimSequence) const
{
	return ((CurveName != NAME_None) && InAnimSequence->HasCurveData(CurveName));
}

float FCachedFloatCurve::GetValueAtPosition(const UAnimSequenceBase* InAnimSequence, const float& InPosition) const
{
	return InAnimSequence->EvaluateCurveData(CurveName, FAnimExtractContext(static_cast<const double>(InPosition)));
}

const FFloatCurve* FCachedFloatCurve::GetFloatCurve(const UAnimSequenceBase* InAnimSequence) const
{
	if (InAnimSequence)
	{
		return static_cast<const FFloatCurve*>(InAnimSequence->GetCurveData().GetCurveData(CurveName));
	}

	return nullptr;
}

/////////////////////////////////////////////////////
// FRawCurveTracks

void FRawCurveTracks::EvaluateCurveData( FBlendedCurve& Curves, float CurrentTime ) const
{
	SCOPE_CYCLE_COUNTER(STAT_EvalRawCurveData);
	
	auto GetNameFromIndex = [this](int32 InCurveIndex)
	{
		return FloatCurves[InCurveIndex].GetName();
	};

	auto GetValueFromIndex = [this, CurrentTime](int32 InCurveIndex)
	{
		return FloatCurves[InCurveIndex].Evaluate(CurrentTime);
	};
	
	// evaluate the curve data at the CurrentTime and add to Instance
	UE::Anim::FCurveUtils::BuildUnsorted(Curves, FloatCurves.Num(), GetNameFromIndex, GetValueFromIndex, Curves.GetFilter());
}

#if WITH_EDITOR
/**
 * Since we don't care about blending, we just change this decoration to OutCurves
 * @TODO : Fix this if we're saving vectorcurves and blending
 */
void FRawCurveTracks::EvaluateTransformCurveData(USkeleton * Skeleton, TMap<FName, FTransform>&OutCurves, float CurrentTime, float BlendWeight) const
{
	check (Skeleton);
	// evaluate the curve data at the CurrentTime and add to Instance
	for(auto CurveIter = TransformCurves.CreateConstIterator(); CurveIter; ++CurveIter)
	{
		const FTransformCurve& Curve = *CurveIter;

		// if disabled, do not handle
		if (Curve.GetCurveTypeFlag(AACF_Disabled))
		{
			continue;
		}

		// Add or retrieve curve
		FName CurveName = Curve.GetName();
		
		// note we're not checking Curve.GetCurveTypeFlags() yet
		FTransform & Value = OutCurves.FindOrAdd(CurveName);
		Value = Curve.Evaluate(CurrentTime, BlendWeight);
	}
}
#endif

const FAnimCurveBase * FRawCurveTracks::GetCurveData(FName Name, ERawCurveTrackTypes SupportedCurveType /*= FloatType*/) const
{
	switch (SupportedCurveType)
	{
#if WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Vector:
		return GetCurveDataImpl<FVectorCurve>(VectorCurves, Name);
	case ERawCurveTrackTypes::RCT_Transform:
		return GetCurveDataImpl<FTransformCurve>(TransformCurves, Name);
#endif // WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Float:
	default:
		return GetCurveDataImpl<FFloatCurve>(FloatCurves, Name);
	}
}

FAnimCurveBase * FRawCurveTracks::GetCurveData(FName Name, ERawCurveTrackTypes SupportedCurveType /*= FloatType*/)
{
	switch (SupportedCurveType)
	{
#if WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Vector:
		return GetCurveDataImpl<FVectorCurve>(VectorCurves, Name);
	case ERawCurveTrackTypes::RCT_Transform:
		return GetCurveDataImpl<FTransformCurve>(TransformCurves, Name);
#endif // WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Float:
	default:
		return GetCurveDataImpl<FFloatCurve>(FloatCurves, Name);
	}
}

bool FRawCurveTracks::DeleteCurveData(const FName& CurveToDelete, ERawCurveTrackTypes SupportedCurveType /*= FloatType*/)
{
	switch(SupportedCurveType)
	{
#if WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Vector:
		return DeleteCurveDataImpl<FVectorCurve>(VectorCurves, CurveToDelete);
	case ERawCurveTrackTypes::RCT_Transform:
		return DeleteCurveDataImpl<FTransformCurve>(TransformCurves, CurveToDelete);
#endif // WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Float:
	default:
		return DeleteCurveDataImpl<FFloatCurve>(FloatCurves, CurveToDelete);
	}
}

void FRawCurveTracks::DeleteAllCurveData(ERawCurveTrackTypes SupportedCurveType /*= FloatType*/)
{
	switch(SupportedCurveType)
	{
#if WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Vector:
		VectorCurves.Empty();
		break;
	case ERawCurveTrackTypes::RCT_Transform:
		TransformCurves.Empty();
		break;
#endif // WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Float:
	default:
		FloatCurves.Empty();
		break;
	}
}

#if WITH_EDITOR
void FRawCurveTracks::AddFloatCurveKey(const FName& NewCurve, int32 CurveFlags, float Time, float Value)
{
	FFloatCurve* FloatCurve = GetCurveDataImpl<FFloatCurve>(FloatCurves, NewCurve);
	if (FloatCurve == nullptr)
	{
		AddCurveData(NewCurve, CurveFlags, ERawCurveTrackTypes::RCT_Float);
		FloatCurve = GetCurveDataImpl<FFloatCurve>(FloatCurves, NewCurve);
	}

	if (FloatCurve->GetCurveTypeFlags() != CurveFlags)
	{
		FloatCurve->SetCurveTypeFlags(FloatCurve->GetCurveTypeFlags() | CurveFlags);
	}

	FloatCurve->UpdateOrAddKey(Value, Time);
}

void FRawCurveTracks::RemoveRedundantKeys(float Tolerance /*= UE_SMALL_NUMBER*/, FFrameRate SampleRate /*= FFrameRate(0,0)*/ )
{
	for (auto CurveIter = FloatCurves.CreateIterator(); CurveIter; ++CurveIter)
	{
		FFloatCurve& Curve = *CurveIter;
		Curve.FloatCurve.RemoveRedundantKeys(Tolerance, SampleRate);
	}
}
#endif

bool FRawCurveTracks::AddCurveData(const FName& NewCurve, int32 CurveFlags /*= ACF_DefaultCurve*/, ERawCurveTrackTypes SupportedCurveType /*= FloatType*/)
{
	switch(SupportedCurveType)
	{
#if WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Vector:
		return AddCurveDataImpl<FVectorCurve>(VectorCurves, NewCurve, CurveFlags);
	case ERawCurveTrackTypes::RCT_Transform:
		return AddCurveDataImpl<FTransformCurve>(TransformCurves, NewCurve, CurveFlags);
#endif // WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Float:
	default:
		return AddCurveDataImpl<FFloatCurve>(FloatCurves, NewCurve, CurveFlags);
	}
}

void FRawCurveTracks::Resize(float TotalLength, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime)
{
	for (auto& Curve: FloatCurves)
	{
		Curve.Resize(TotalLength, bInsert, OldStartTime, OldEndTime);
	}

#if WITH_EDITORONLY_DATA
	for(auto& Curve: VectorCurves)
	{
		Curve.Resize(TotalLength, bInsert, OldStartTime, OldEndTime);
	}

	for(auto& Curve: TransformCurves)
	{
		Curve.Resize(TotalLength, bInsert, OldStartTime, OldEndTime);
	}
#endif
}

void FRawCurveTracks::PostSerializeFixup(FArchive& Ar)
{
	// @TODO: If we're about to serialize vector curve, add here
	for(FFloatCurve& Curve : FloatCurves)
	{
		Curve.PostSerializeFixup(Ar);
	}
#if WITH_EDITORONLY_DATA
	if( !Ar.IsCooking() )
	{
		if( Ar.UEVer() >= VER_UE4_ANIMATION_ADD_TRACKCURVES )
		{
			for( FTransformCurve& Curve : TransformCurves )
			{
				Curve.PostSerializeFixup( Ar );
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

bool FRawCurveTracks::DuplicateCurveData(const FName& CurveToCopy, const FName& NewCurve, ERawCurveTrackTypes SupportedCurveType /*= FloatType*/)
{
	switch(SupportedCurveType)
	{
#if WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Vector:
		return DuplicateCurveDataImpl<FVectorCurve>(VectorCurves, CurveToCopy, NewCurve);
	case ERawCurveTrackTypes::RCT_Transform:
		return DuplicateCurveDataImpl<FTransformCurve>(TransformCurves, CurveToCopy, NewCurve);
#endif // WITH_EDITOR
	case ERawCurveTrackTypes::RCT_Float:
	default:
		return DuplicateCurveDataImpl<FFloatCurve>(FloatCurves, CurveToCopy, NewCurve);
	}
}

///////////////////////////////////
// @TODO: REFACTOR THIS IF WE'RE SERIALIZING VECTOR CURVES
//
// implementation template functions to accomodate FloatCurve and VectorCurve
// for now vector curve isn't used in run-time, so it's useless outside of editor
// so just to reduce cost of run-time, functionality is split. 
// this split worries me a bit because if the name conflict happens this will break down w.r.t. smart naming
// currently vector curve is not saved and not evaluated, so it will be okay since the name doesn't matter much, 
// but this has to be refactored once we'd like to move onto serialize
///////////////////////////////////
template <typename DataType>
DataType * FRawCurveTracks::GetCurveDataImpl(TArray<DataType> & Curves, FName Name)
{
	for (DataType& Curve : Curves)
	{
		if (Curve.GetName() == Name)
		{
			return &Curve;
		}
	}

	return nullptr;
}

template <typename DataType>
const DataType * FRawCurveTracks::GetCurveDataImpl(const TArray<DataType> & Curves, FName Name) const
{
	for (const DataType& Curve : Curves)
	{
		if (Curve.GetName() == Name)
		{
			return &Curve;
		}
	}

	return nullptr;
}

template <typename DataType>
bool FRawCurveTracks::DeleteCurveDataImpl(TArray<DataType> & Curves, const FName& CurveToDelete)
{
	for(int32 Idx = 0; Idx < Curves.Num(); ++Idx)
	{
		if(Curves[Idx].GetName() == CurveToDelete)
		{
			Curves.RemoveAt(Idx);
			return true;
		}
	}

	return false;
}

template <typename DataType>
bool FRawCurveTracks::AddCurveDataImpl(TArray<DataType> & Curves, const FName& NewCurve, int32 CurveFlags)
{
	if(GetCurveDataImpl<DataType>(Curves, NewCurve) == NULL)
	{
		Curves.Add(DataType(NewCurve, CurveFlags));
		return true;
	}
	return false;
}

template <typename DataType>
bool FRawCurveTracks::DuplicateCurveDataImpl(TArray<DataType> & Curves, const FName& CurveToCopy, const FName& NewCurve)
{
	DataType* ExistingCurve = GetCurveDataImpl<DataType>(Curves, CurveToCopy);
	if(ExistingCurve && GetCurveDataImpl<DataType>(Curves, NewCurve) == NULL)
	{
		// Add the curve to the track and set its data to the existing curve
		Curves.Add(DataType(NewCurve, ExistingCurve->GetCurveTypeFlags()));
		Curves.Last().CopyCurve(*ExistingCurve);

		return true;
	}
	return false;
}

FArchive& operator<<(FArchive& Ar, FRawCurveTracks& D)
{
	UScriptStruct* StaticStruct = FRawCurveTracks::StaticStruct();
	StaticStruct->SerializeTaggedProperties(Ar, (uint8*)&D, StaticStruct, nullptr);
	// do not call custom serialize that relies on version number. The Archive version doesn't exists on this. 
	return Ar;
}

void FBlendedCurve::InitFrom(const FBoneContainer& InBoneContainer)
{
	TBaseBlendedCurve<FAnimStackAllocator>::SetFilter(&InBoneContainer.GetCurveFilter());
}

void FBlendedHeapCurve::InitFrom(const FBoneContainer& InBoneContainer)
{
	TBaseBlendedCurve<FDefaultAllocator>::SetFilter(&InBoneContainer.GetCurveFilter()); 
}
