// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimData/IAnimationDataController.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IAnimationDataController)

void IAnimationDataController::ReportMessage(const UObject* ErrorObject, const FText& InMessage, ELogVerbosity::Type LogVerbosity)
{
#if WITH_EDITOR
	FString Message = InMessage.ToString();
	if (ErrorObject != nullptr)
	{
		if (const UPackage* Package = Cast<UPackage>(ErrorObject->GetOutermost()))
		{
			Message = FString::Printf(TEXT("%s : %s"), *Package->GetPathName(), *Message);
		}
	}

	FScriptExceptionHandler::Get().HandleException(LogVerbosity, *Message, *FString());
#endif
}
