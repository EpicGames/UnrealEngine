// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Features/IModularFeatures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IAnimationDataModel)

namespace UE::Anim::DataModel
{
	UClass* IAnimationDataModels::FindClassForAnimationAsset(UAnimSequenceBase* AnimSequenceBase)	
	{
		TArray<IAnimationDataModels*> DataModels = IModularFeatures::Get().GetModularFeatureImplementations<IAnimationDataModels>(IAnimationDataModels::GetModularFeatureName());
		UClass* TargetClass = UAnimDataModel::StaticClass();

		for (const IAnimationDataModels* Model : DataModels)
		{
			if (Model)
			{
				if (UClass* SubClass = Model->GetModelClass(AnimSequenceBase))
				{
					TargetClass = SubClass;
				}
			}
		}

		return TargetClass;
	}	
}
