// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimData/AnimDataModel.h"

#include "Animation/AnimData/CurveIdentifier.h"
#include "Animation/AnimSequence.h"

#include "Algo/Accumulate.h"
#include "Animation/AnimDataModelHasher.h"
#include "EngineLogs.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Misc/SecureHash.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDataModel)

#if WITH_EDITOR
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/AttributesRuntime.h"
#include "AnimationRuntime.h"
#include "AnimEncoding.h"
#include "IAnimationDataControllerModule.h"
#include "Modules/ModuleManager.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AnimDataModel"

void UAnimDataModel::PostLoad()
{
	UObject::PostLoad();

	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::ForceUpdateAnimationAssetCurveTangents)
	{
		// Forcefully AutoSetTangents to fix-up any imported sequences pre the fix for flattening first/last key leave/arrive tangents
		GetNotifier().Notify(EAnimDataModelNotifyType::BracketOpened);
		for (FFloatCurve& FloatCurve : CurveData.FloatCurves)
		{
			FloatCurve.FloatCurve.AutoSetTangents();
			FCurvePayload Payload;
			Payload.Identifier = FAnimationCurveIdentifier(FloatCurve.GetName(), ERawCurveTrackTypes::RCT_Float);
			GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);
		}
		GetNotifier().Notify(EAnimDataModelNotifyType::BracketClosed);
	}

	const bool bHasBoneTracks = BoneAnimationTracks.Num() > 0;

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SingleFrameAndKeyAnimModel)
	{
		if (bHasBoneTracks)
		{
			// Number of keys was used directly rather than Max(Value,2), as a single _frame_ animation should always have two _keys_ 
			const int32 ActualNumKeys = BoneAnimationTracks[0].InternalTrackData.PosKeys.Num();
			if (ActualNumKeys == 1 && NumberOfKeys == 2)
			{
				auto AddKey = [this](auto& Keys)
				{
					const auto KeyZero = Keys[0];
					Keys.Add(KeyZero);
					ensure(Keys.Num() == NumberOfKeys);
				};
				
				for (FBoneAnimationTrack& BoneTrack : BoneAnimationTracks)
				{
					AddKey(BoneTrack.InternalTrackData.PosKeys);
					AddKey(BoneTrack.InternalTrackData.RotKeys);
					AddKey(BoneTrack.InternalTrackData.ScaleKeys);
				}

				GetNotifier().Notify(EAnimDataModelNotifyType::TrackChanged);
			}
		}
	}
	
	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::ReintroduceAnimationDataModelInterface)
	{		
		const double NumFramesAsSeconds = FrameRate.AsSeconds(NumberOfFrames);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const float CachedPlayLength = PlayLength;
		if (static_cast<float>(NumFramesAsSeconds) != PlayLength)
		{
			if (bHasBoneTracks)
			{
				// Number of keys was rounded up from sequence length, so set play length from stored number of bone keys 
				const int32 ActualNumKeys = bHasBoneTracks ? FMath::Min(BoneAnimationTracks[0].InternalTrackData.PosKeys.Num(), NumberOfKeys) : NumberOfKeys;
				if (ActualNumKeys != NumberOfKeys)
				{
					PlayLength = FrameRate.AsSeconds(ActualNumKeys - 1);
					NumberOfFrames = ActualNumKeys - 1;
					NumberOfKeys = ActualNumKeys;

					IAnimationDataController::ReportObjectWarningf(this, LOCTEXT("AnimDataModelPlayLengthAlteredToBones", "Playable length {0} has been updated according to number of frames {1} and frame-rate {2}, new length: {3}"), FText::AsNumber(CachedPlayLength), FText::AsNumber(GetNumberOfFrames()), FrameRate.ToPrettyText(), FText::AsNumber(PlayLength));
				}
			}
			else
			{
				// No way to determine 'correct' length so set current PlayLength value to # of frames * frame-rate
				PlayLength = FrameRate.AsSeconds(NumberOfFrames);
			}
			GetNotifier().Notify(EAnimDataModelNotifyType::Populated);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UAnimDataModel::PostDuplicate(bool bDuplicateForPIE)
{
	UObject::PostDuplicate(bDuplicateForPIE);

	GetNotifier().Notify(EAnimDataModelNotifyType::Populated);
}

const TArray<FBoneAnimationTrack>& UAnimDataModel::GetBoneAnimationTracks() const
{
	return BoneAnimationTracks;
}

const FBoneAnimationTrack& UAnimDataModel::GetBoneTrackByIndex(int32 TrackIndex) const
{
	checkf(BoneAnimationTracks.IsValidIndex(TrackIndex), TEXT("Unable to find animation track by index"));
	return BoneAnimationTracks[TrackIndex];
}

const FBoneAnimationTrack& UAnimDataModel::GetBoneTrackByName(FName TrackName) const
{
	const FBoneAnimationTrack* TrackPtr = BoneAnimationTracks.FindByPredicate([TrackName](const FBoneAnimationTrack& Track)
	{
		return Track.Name == TrackName;
	});

	checkf(TrackPtr != nullptr, TEXT("Unable to find animation track by name"));

	return *TrackPtr;
}

int32 UAnimDataModel::GetBoneTrackIndex(const FBoneAnimationTrack& Track) const
{
	return BoneAnimationTracks.IndexOfByPredicate([&Track](const FBoneAnimationTrack& SearchTrack)
	{
		return SearchTrack.Name == Track.Name;
	});
}

int32 UAnimDataModel::GetBoneTrackIndexByName(FName TrackName) const
{
	return BoneAnimationTracks.IndexOfByPredicate([TrackName](const FBoneAnimationTrack& SearchTrack)
	{
		return SearchTrack.Name == TrackName;
	});
}

double UAnimDataModel::GetPlayLength() const
{
	return FrameRate.AsSeconds(NumberOfFrames);
}

int32 UAnimDataModel::GetNumberOfFrames() const
{
	return NumberOfFrames;
}

int32 UAnimDataModel::GetNumberOfKeys() const
{
	return NumberOfKeys;
}

FFrameRate UAnimDataModel::GetFrameRate() const
{
	return FrameRate;
}

bool UAnimDataModel::IsValidBoneTrackIndex(int32 TrackIndex) const
{
	return BoneAnimationTracks.IsValidIndex(TrackIndex);
}

int32 UAnimDataModel::GetNumBoneTracks() const
{
	return BoneAnimationTracks.Num();
}

UAnimSequence* UAnimDataModel::GetAnimationSequence() const
{
	UObject* AnimObject = static_cast<UObject*>(FindObjectWithOuter(GetOuter(), UAnimSequence::StaticClass()));
	return CastChecked<UAnimSequence>(AnimObject ? AnimObject : GetOuter());
}

void UAnimDataModel::GetBoneTrackNames(TArray<FName>& OutNames) const
{
	Algo::Transform(BoneAnimationTracks, OutNames, [](const FBoneAnimationTrack& Track)
	{
		return Track.Name; 
	});
}

FTransform UAnimDataModel::GetBoneTrackTransform(FName TrackName, const FFrameNumber& FrameNumber) const
{
	const FBoneAnimationTrack* Track = GetBoneAnimationTracks().FindByPredicate([TrackName](const FBoneAnimationTrack& Track)
	{
		return Track.Name == TrackName;
	});

	if (Track)
	{
		const int32 KeyIndex = FrameNumber.Value;
		if (Track->InternalTrackData.PosKeys.IsValidIndex(KeyIndex) &&
			Track->InternalTrackData.RotKeys.IsValidIndex(KeyIndex) &&
			Track->InternalTrackData.ScaleKeys.IsValidIndex(KeyIndex))
		{
			return FTransform(FQuat(Track->InternalTrackData.RotKeys[KeyIndex]),
							FVector(Track->InternalTrackData.PosKeys[KeyIndex]),
							FVector(Track->InternalTrackData.ScaleKeys[KeyIndex])
			);
		}
	}

	return FTransform::Identity;
}

void UAnimDataModel::GetBoneTrackTransforms(FName TrackName, const TArray<FFrameNumber>& FrameNumbers, TArray<FTransform>& OutTransforms) const
{
	const FBoneAnimationTrack* Track = GetBoneAnimationTracks().FindByPredicate([TrackName](const FBoneAnimationTrack& Track)
	{
		return Track.Name == TrackName;
	});

	OutTransforms.SetNum(FrameNumbers.Num());

	if (Track)
	{
		for (int32 EntryIndex = 0; EntryIndex < FrameNumbers.Num(); ++EntryIndex)
		{
			OutTransforms[EntryIndex] = GetBoneTrackTransform(TrackName, FrameNumbers[EntryIndex]);
		}
	}
}

void UAnimDataModel::GetBoneTrackTransforms(FName TrackName, TArray<FTransform>& OutTransforms) const
{
	const FBoneAnimationTrack* Track = GetBoneAnimationTracks().FindByPredicate([TrackName](const FBoneAnimationTrack& Track)
	{
		return Track.Name == TrackName;
	});
	
	OutTransforms.SetNum(NumberOfKeys);

	if (Track)
	{
		for (int32 KeyIndex = 0; KeyIndex < NumberOfKeys; ++KeyIndex)
		{
			OutTransforms[KeyIndex] = GetBoneTrackTransform(TrackName, FFrameNumber(KeyIndex));
		}
	}
}

void UAnimDataModel::GetBoneTracksTransform(const TArray<FName>& TrackNames, const FFrameNumber& FrameNumber, TArray<FTransform>& OutTransforms) const
{	
	OutTransforms.SetNum(TrackNames.Num());
	for (int32 EntryIndex = 0; EntryIndex < TrackNames.Num(); ++EntryIndex)
	{
		OutTransforms[EntryIndex] = GetBoneTrackTransform(TrackNames[EntryIndex], FrameNumber);
	}
}

FTransform UAnimDataModel::EvaluateBoneTrackTransform(FName TrackName, const FFrameTime& FrameTime, const EAnimInterpolationType& Interpolation) const
{
	const float Alpha = Interpolation == EAnimInterpolationType::Step ? FMath::RoundToFloat(FrameTime.GetSubFrame()) : FrameTime.GetSubFrame();

	if (FMath::IsNearlyEqual(Alpha, 1.0f))
	{
		return GetBoneTrackTransform(TrackName, FrameTime.CeilToFrame());
	}
	else if (FMath::IsNearlyZero(Alpha))
	{
		return GetBoneTrackTransform(TrackName, FrameTime.FloorToFrame());
	}
	
	const FTransform From = GetBoneTrackTransform(TrackName, FrameTime.FloorToFrame());
	const FTransform To = GetBoneTrackTransform(TrackName, FrameTime.CeilToFrame());

	FTransform Blend;
	Blend.Blend(From, To, Alpha);
	return Blend;
}

bool UAnimDataModel::IsValidBoneTrackName(const FName& TrackName) const
{
	return BoneAnimationTracks.ContainsByPredicate([&TrackName](const FBoneAnimationTrack& SearchTrack)
	{
		return SearchTrack.Name == TrackName;
	});
}

const FAnimationCurveData& UAnimDataModel::GetCurveData() const
{
	return CurveData;
}

void UAnimDataModel::IterateBoneKeys(const FName& BoneName, TFunction<bool(const FVector3f& Pos, const FQuat4f&, const FVector3f, const FFrameNumber&)> IterationFunction) const
{
	if (const FBoneAnimationTrack* Track = BoneAnimationTracks.FindByPredicate([BoneName](const FBoneAnimationTrack& SearchTrack) { return SearchTrack.Name == BoneName; }))
	{
		for (int32 KeyIndex = 0; KeyIndex < GetNumberOfKeys(); ++KeyIndex)
		{
			if(!IterationFunction(Track->InternalTrackData.PosKeys[KeyIndex], Track->InternalTrackData.RotKeys[KeyIndex], Track->InternalTrackData.ScaleKeys[KeyIndex], KeyIndex))
			{
				return;
			}
		}
	}
}

IAnimationDataModel::FModelNotifier& UAnimDataModel::GetNotifier()
{
	if (!Notifier)
	{
		Notifier.Reset(new IAnimationDataModel::FModelNotifier(this));
	}
	
	return *Notifier.Get();
}

int32 UAnimDataModel::GetNumberOfTransformCurves() const
{
	return CurveData.TransformCurves.Num();
}

int32 UAnimDataModel::GetNumberOfFloatCurves() const
{
	return CurveData.FloatCurves.Num();
}

const TArray<FFloatCurve>& UAnimDataModel::GetFloatCurves() const
{
	return CurveData.FloatCurves;
}

const TArray<struct FTransformCurve>& UAnimDataModel::GetTransformCurves() const
{
	return CurveData.TransformCurves;
}

const FFloatCurve* UAnimDataModel::FindFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	ensure(CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Float);
	for (const FFloatCurve& FloatCurve : CurveData.FloatCurves)
	{
		if (FloatCurve.GetName() == CurveIdentifier.CurveName)
		{
			return &FloatCurve;
		}
	}

	return nullptr;
}

const FTransformCurve* UAnimDataModel::FindTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	ensure(CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Transform);
	for (const FTransformCurve& TransformCurve : CurveData.TransformCurves)
	{
		if (TransformCurve.GetName() == CurveIdentifier.CurveName)
		{
			return &TransformCurve;
		}
	}

	return nullptr;
}

const FRichCurve* UAnimDataModel::FindRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FRichCurve* RichCurve = nullptr;

	if (CurveIdentifier.IsValid())
	{
		if (CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			const FFloatCurve* Curve = FindFloatCurve(CurveIdentifier);
			if (Curve)
			{
				RichCurve = &Curve->FloatCurve;
			}
		}
		else if (CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Transform)
		{
			if (CurveIdentifier.Channel != ETransformCurveChannel::Invalid && CurveIdentifier.Axis != EVectorCurveChannel::Invalid)
			{
				// Dealing with transform curve
				const FTransformCurve* TransformCurve = FindTransformCurve(CurveIdentifier);
				if (TransformCurve)
				{
					const FVectorCurve* VectorCurve = TransformCurve->GetVectorCurveByIndex((int32)CurveIdentifier.Channel);
					if (VectorCurve)
					{
						RichCurve = &VectorCurve->FloatCurves[(int32)CurveIdentifier.Axis];
					}
				}

			}
		}
	}

	return RichCurve;
}

const FAnimCurveBase& UAnimDataModel::GetCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FAnimCurveBase* CurvePtr = FindCurve(CurveIdentifier);
	checkf(CurvePtr, TEXT("Tried to retrieve non-existing curve"));
	return *CurvePtr;
}

const FFloatCurve& UAnimDataModel::GetFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FFloatCurve* CurvePtr = FindFloatCurve(CurveIdentifier);
	checkf(CurvePtr, TEXT("Tried to retrieve non-existing curve"));
	return *CurvePtr;
}

const FTransformCurve& UAnimDataModel::GetTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FTransformCurve* CurvePtr = FindTransformCurve(CurveIdentifier);
	checkf(CurvePtr, TEXT("Tried to retrieve non-existing curve"));
	return *CurvePtr;
}

const FRichCurve& UAnimDataModel::GetRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FRichCurve* CurvePtr = FindRichCurve(CurveIdentifier);
	checkf(CurvePtr, TEXT("Tried to retrieve non-existing curve"));
	return *CurvePtr;
}

TArrayView<const FAnimatedBoneAttribute> UAnimDataModel::GetAttributes() const
{
	return AnimatedBoneAttributes;
}

int32 UAnimDataModel::GetNumberOfAttributes() const
{
	return AnimatedBoneAttributes.Num();
}

int32 UAnimDataModel::GetNumberOfAttributesForBoneIndex(const int32 BoneIndex) const
{
	// Sum up total number of attributes with provided bone index
	const int32 NumberOfBoneAttributes = Algo::Accumulate<int32>(AnimatedBoneAttributes, 0, [BoneIndex](int32 Sum, const FAnimatedBoneAttribute& Attribute) -> int32
	{
		Sum += Attribute.Identifier.GetBoneIndex() == BoneIndex ? 1 : 0;
		return Sum;
	});
	return NumberOfBoneAttributes;
}

void UAnimDataModel::GetAttributesForBone(const FName& BoneName, TArray<const FAnimatedBoneAttribute*>& OutBoneAttributes) const
{
	Algo::TransformIf(AnimatedBoneAttributes, OutBoneAttributes, [BoneName](const FAnimatedBoneAttribute& Attribute) -> bool
	{
		return Attribute.Identifier.GetBoneName() == BoneName;
	},
	[](const FAnimatedBoneAttribute& Attribute) 
	{
		return &Attribute;
	});
}

const FAnimatedBoneAttribute& UAnimDataModel::GetAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const
{
	const FAnimatedBoneAttribute* AttributePtr = FindAttribute(AttributeIdentifier);
	checkf(AttributePtr, TEXT("Unable to find attribute for provided identifier"));

	return *AttributePtr;
}

const FAnimatedBoneAttribute* UAnimDataModel::FindAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const
{
	return AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute)
	{
		return Attribute.Identifier == AttributeIdentifier;
	});
}

const FAnimCurveBase* UAnimDataModel::FindCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	switch (CurveIdentifier.CurveType)
	{
		case ERawCurveTrackTypes::RCT_Float:
			return FindFloatCurve(CurveIdentifier);
		case ERawCurveTrackTypes::RCT_Transform:
			return FindTransformCurve(CurveIdentifier);
	}

	return nullptr;
}

template <typename HasherType>
void UAnimDataModel::GenerateStateHash(HasherType& Hasher, const FGuidGenerationSettings& InSettings) const
{
	if (InSettings.bIncludeTimingData)
	{		
		Hasher.UpdateData(GetNumberOfFrames(), TEXT("FR"));
	}

	if (InSettings.bIncludeBoneData)
	{
		Hasher.BeginObject(TEXT("BAT"));
		for (const FBoneAnimationTrack& Track : BoneAnimationTracks)
		{
			Hasher.BeginObject();
			Hasher.UpdateArray(Track.InternalTrackData.PosKeys, TEXT("P"));
			Hasher.UpdateArray(Track.InternalTrackData.RotKeys, TEXT("R"));
			Hasher.UpdateArray(Track.InternalTrackData.ScaleKeys, TEXT("S"));
			Hasher.EndObject();
		}
		Hasher.EndObject();
	}

	if (InSettings.bIncludeCurveData)
	{
		Hasher.BeginObject(TEXT("FC"));
		for (const FFloatCurve& Curve : CurveData.FloatCurves)
		{
			Hasher.BeginObject();
			Hasher.UpdateLegacyString(Curve.GetName().ToString(), TEXT("N"));
			Hasher.UpdateRichCurve(Curve.FloatCurve, TEXT("C"));
			Hasher.EndObject();
		}
		Hasher.EndObject();

		Hasher.UpdateTransformCurves(CurveData.TransformCurves, TEXT("TC"));
	}

	if (InSettings.bIncludeAttributeData)
	{
		Hasher.UpdateAnimatedBoneAttributes(AnimatedBoneAttributes, TEXT("ABA"));
	}
}

FGuid UAnimDataModel::GenerateGuid(const FGuidGenerationSettings& InSettings) const
{
	UE::Anim::DataModel::FHasherSha Hasher;
	GenerateStateHash(Hasher, InSettings);
	return Hasher.FinalGuid();
}

#if WITH_EDITOR
FString UAnimDataModel::GenerateDebugStateString() const
{
	UE::Anim::DataModel::FHasherCopyToText Hasher;
	GenerateStateHash(Hasher, FGuidGenerationSettings());
	return Hasher.GetString();
}
#endif

TScriptInterface<IAnimationDataController> UAnimDataModel::GetController()
{
	TScriptInterface<IAnimationDataController> Controller = nullptr;
#if WITH_EDITOR
	IAnimationDataControllerModule& ControllerModule = FModuleManager::Get().GetModuleChecked<IAnimationDataControllerModule>("AnimationDataController");
	Controller = ControllerModule.GetController();
	Controller->SetModel(this);
#endif // WITH_EDITOR

	return Controller;
}

#if WITH_EDITOR
FTransform ExtractTransformForKeyIndex(int32 Key, const FRawAnimSequenceTrack& TrackToExtract)
{
	static const FVector DefaultScale3D = FVector(1.f);
	const bool bHasScaleKey = TrackToExtract.ScaleKeys.Num() > 0;

	const int32 PosKeyIndex = FMath::Min(Key, TrackToExtract.PosKeys.Num() - 1);
	const int32 RotKeyIndex = FMath::Min(Key, TrackToExtract.RotKeys.Num() - 1);
	if (bHasScaleKey)
	{
		const int32 ScaleKeyIndex = FMath::Min(Key, TrackToExtract.ScaleKeys.Num() - 1);
		return FTransform(FQuat(TrackToExtract.RotKeys[RotKeyIndex]), FVector(TrackToExtract.PosKeys[PosKeyIndex]), FVector(TrackToExtract.ScaleKeys[ScaleKeyIndex]));
	}
	else
	{
		return FTransform(FQuat(TrackToExtract.RotKeys[RotKeyIndex]), FVector(TrackToExtract.PosKeys[PosKeyIndex]), DefaultScale3D);
	}
}

template<bool bInterpolateT>
void ExtractPose(const TArray<FBoneAnimationTrack>& BoneAnimationTracks, const TMap<FName, const FTransformCurve*>& OverrideBoneTransforms, FCompactPose& InOutPose, const int32 KeyIndex1, const int32 KeyIndex2, float Alpha, float TimePerKey, UE::Anim::Retargeting::FRetargetingScope& RetargetingScope)
{
	const int32 NumAnimationTracks = BoneAnimationTracks.Num();
	const FBoneContainer& RequiredBones = InOutPose.GetBoneContainer();

	TArray<FVirtualBoneCompactPoseData>& VBCompactPoseData = UE::Anim::FBuildRawPoseScratchArea::Get().VirtualBoneCompactPoseData;
	VBCompactPoseData = RequiredBones.GetVirtualBoneCompactPoseData();

	FCompactPose Key2Pose;
	if (bInterpolateT)
	{
		Key2Pose.CopyBonesFrom(InOutPose);
	}

	for (const FBoneAnimationTrack& AnimationTrack : BoneAnimationTracks)
	{
		const int32 SkeletonBoneIndex = AnimationTrack.BoneTreeIndex;
		// not sure it's safe to assume that SkeletonBoneIndex can never be INDEX_NONE
		if ((SkeletonBoneIndex != INDEX_NONE) && (SkeletonBoneIndex < MAX_BONES))
		{
			const FCompactPoseBoneIndex PoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonIndex(SkeletonBoneIndex);
			if (PoseBoneIndex != INDEX_NONE)
			{
				for (int32 Idx = 0; Idx < VBCompactPoseData.Num(); ++Idx)
				{
					FVirtualBoneCompactPoseData& VB = VBCompactPoseData[Idx];
					if (PoseBoneIndex == VB.VBIndex)
					{
						// Remove this bone as we have written data for it (false so we dont resize allocation)
						VBCompactPoseData.RemoveAtSwap(Idx, EAllowShrinking::No);
						break; //Modified TArray so must break here
					}
				}
				
				// extract animation
				const FRawAnimSequenceTrack& TrackToExtract = AnimationTrack.InternalTrackData;
				auto OverrideTransform = OverrideBoneTransforms.Find(AnimationTrack.Name);
				{
					// Bail out (with rather wacky data) if data is empty for some reason.
					if (TrackToExtract.PosKeys.Num() == 0 || TrackToExtract.RotKeys.Num() == 0)
					{
						InOutPose[PoseBoneIndex].SetIdentity();

						if (bInterpolateT)
						{
							Key2Pose[PoseBoneIndex].SetIdentity();
						}
					}
					else
					{
						InOutPose[PoseBoneIndex] = ExtractTransformForKeyIndex(KeyIndex1, TrackToExtract);

						if (bInterpolateT)
						{
							Key2Pose[PoseBoneIndex] = ExtractTransformForKeyIndex(KeyIndex2, TrackToExtract);
						}
					}
				}

				if (OverrideTransform)
				{
					{
						const FTransform PoseOneAdditive = (*OverrideTransform)->Evaluate(KeyIndex1 * TimePerKey, 1.f);
						const FTransform PoseOneLocalTransform = InOutPose[PoseBoneIndex];
						InOutPose[PoseBoneIndex].SetRotation(PoseOneLocalTransform.GetRotation() * PoseOneAdditive.GetRotation());
						InOutPose[PoseBoneIndex].SetTranslation(PoseOneLocalTransform.TransformPosition(PoseOneAdditive.GetTranslation()));
						InOutPose[PoseBoneIndex].SetScale3D(PoseOneLocalTransform.GetScale3D() * PoseOneAdditive.GetScale3D());
					}
					

					if (bInterpolateT)
					{
						const FTransform PoseTwoAdditive = (*OverrideTransform)->Evaluate(KeyIndex2 * TimePerKey, 1.f);
						const FTransform PoseTwoLocalTransform = Key2Pose[PoseBoneIndex];						
						Key2Pose[PoseBoneIndex].SetRotation(PoseTwoLocalTransform.GetRotation() * PoseTwoAdditive.GetRotation());
						Key2Pose[PoseBoneIndex].SetTranslation(PoseTwoLocalTransform.TransformPosition(PoseTwoAdditive.GetTranslation()));
						Key2Pose[PoseBoneIndex].SetScale3D(PoseTwoLocalTransform.GetScale3D() * PoseTwoAdditive.GetScale3D());
					}
				}
				RetargetingScope.AddTrackedBone(PoseBoneIndex, SkeletonBoneIndex);
			}
		}
	}

	//Build Virtual Bones
	if (VBCompactPoseData.Num() > 0)
	{
		FCSPose<FCompactPose> CSPose1;
		CSPose1.InitPose(InOutPose);

		FCSPose<FCompactPose> CSPose2;
		if (bInterpolateT)
		{
			CSPose2.InitPose(Key2Pose);
		}

		for (FVirtualBoneCompactPoseData& VB : VBCompactPoseData)
		{
			FTransform Source = CSPose1.GetComponentSpaceTransform(VB.SourceIndex);
			FTransform Target = CSPose1.GetComponentSpaceTransform(VB.TargetIndex);
			InOutPose[VB.VBIndex] = Target.GetRelativeTransform(Source);

			if (bInterpolateT)
			{
				FTransform Source2 = CSPose2.GetComponentSpaceTransform(VB.SourceIndex);
				FTransform Target2 = CSPose2.GetComponentSpaceTransform(VB.TargetIndex);
				Key2Pose[VB.VBIndex] = Target2.GetRelativeTransform(Source2);
			}
		}
	}

	if (bInterpolateT)
	{
		for (FCompactPoseBoneIndex BoneIndex : InOutPose.ForEachBoneIndex())
		{
			InOutPose[BoneIndex].Blend(InOutPose[BoneIndex], Key2Pose[BoneIndex], Alpha);
		}
	}
}

void UAnimDataModel::Evaluate(FAnimationPoseData& InOutPoseData, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const
{
	const float SequenceLength = GetPlayLength();

	const double Time = EvaluationContext.SampleFrameRate.AsSeconds(EvaluationContext.SampleTime);
	const EAnimInterpolationType& InterpolationType = EvaluationContext.InterpolationType;
	FCompactPose& OutPose = InOutPoseData.GetPose();
	
	// Generate keys to interpolate between
	int32 KeyIndex1, KeyIndex2;
	float Alpha;
	FAnimationRuntime::GetKeyIndicesFromTime(KeyIndex1, KeyIndex2, Alpha, Time, FrameRate, NumberOfKeys);

	if (InterpolationType == EAnimInterpolationType::Step)
	{
		// Force stepping between keys
		Alpha = 0.f;
	}

	bool bShouldInterpolate = true;

	if (Alpha < UE_KINDA_SMALL_NUMBER)
	{
		Alpha = 0.f;
		bShouldInterpolate = false;
	}
	else if (Alpha > 1.f - UE_KINDA_SMALL_NUMBER)
	{
		bShouldInterpolate = false;
		KeyIndex1 = KeyIndex2;
	}

	// Evaluate animation float curve data
	UE::Anim::EvaluateFloatCurvesFromModel(this, InOutPoseData.GetCurve(), Time);

	const double TimePerFrame = FrameRate.AsInterval();
	TMap<FName, const FTransformCurve*> ActiveCurves;

	if (!OutPose.GetBoneContainer().ShouldUseSourceData())
	{
		for (const FTransformCurve& Curve : GetTransformCurves())
		{
			// if disabled, do not handle
			if (Curve.GetCurveTypeFlag(AACF_Disabled))
			{
				continue;
			}

			// note we're not checking Curve.GetCurveTypeFlags() yet
			ActiveCurves.FindOrAdd(Curve.GetName(), &Curve);
		}
	}

	{		
		UE::Anim::Retargeting::FRetargetingScope RetargetingScope(GetSkeleton(), OutPose, EvaluationContext);
		if (bShouldInterpolate)
		{		
			ExtractPose<true>(BoneAnimationTracks, ActiveCurves, OutPose, KeyIndex1, KeyIndex2, Alpha, TimePerFrame, RetargetingScope);
		}
		else
		{
			ExtractPose<false>(BoneAnimationTracks, ActiveCurves, OutPose, KeyIndex1, KeyIndex2, Alpha, TimePerFrame, RetargetingScope);
		}
	}
	

	const FBoneContainer& RequiredBones = InOutPoseData.GetPose().GetBoneContainer();
	for (const FAnimatedBoneAttribute& Attribute : AnimatedBoneAttributes)
	{
		const FCompactPoseBoneIndex PoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonIndex(Attribute.Identifier.GetBoneIndex());
		// Only add attribute if the bone its tied to exists in the currently evaluated set of bones
		if(PoseBoneIndex.IsValid())
		{
			UE::Anim::Attributes::GetAttributeValue(InOutPoseData.GetAttributes(), PoseBoneIndex, Attribute, EvaluationContext.SampleFrameRate.AsSeconds(EvaluationContext.SampleTime));
		}
	}	
}
#endif // WITH_EDITOR

FRichCurve* UAnimDataModel::GetMutableRichCurve(const FAnimationCurveIdentifier& CurveIdentifier)
{
	FRichCurve* RichCurve = nullptr;

	if (CurveIdentifier.IsValid())
	{
		if (CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			FFloatCurve* Curve = FindMutableFloatCurveById(CurveIdentifier);
			if (Curve)
			{
				RichCurve = &Curve->FloatCurve;
			}
		}
		else if (CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Transform)
		{
			if (CurveIdentifier.Channel != ETransformCurveChannel::Invalid && CurveIdentifier.Axis != EVectorCurveChannel::Invalid)
			{
				// Dealing with transform curve
				FTransformCurve* TransformCurve = FindMutableTransformCurveById(CurveIdentifier);
				if (TransformCurve)
				{
					FVectorCurve* VectorCurve = TransformCurve->GetVectorCurveByIndex((int32)CurveIdentifier.Channel);
					if (VectorCurve)
					{
						RichCurve = &VectorCurve->FloatCurves[(int32)CurveIdentifier.Axis];
					}
				}

			}
		}
	}

	return RichCurve;
}

USkeleton* UAnimDataModel::GetSkeleton() const
{
	const UAnimationAsset* AnimationAsset = CastChecked<UAnimationAsset>(GetOuter());	
	checkf(AnimationAsset, TEXT("Unable to retrieve owning AnimationAsset"));

	USkeleton* Skeleton = AnimationAsset->GetSkeleton();
	if (Skeleton == nullptr)
	{
		IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("UnableToFindSkeleton", "Unable to retrieve target USkeleton for Animation Asset ({0})"), FText::FromString(*AnimationAsset->GetPathName()));
	} 

	return Skeleton;
}

FBoneAnimationTrack* UAnimDataModel::FindMutableBoneTrackByName(FName Name)
{
	return BoneAnimationTracks.FindByPredicate([Name](const FBoneAnimationTrack& Track)
	{
		return Track.Name == Name;
	});
}

FBoneAnimationTrack& UAnimDataModel::GetMutableBoneTrackByName(FName Name)
{
	FBoneAnimationTrack* TrackPtr = BoneAnimationTracks.FindByPredicate([Name](const FBoneAnimationTrack& Track)
	{
		return Track.Name == Name;
	});

	checkf(TrackPtr, TEXT("Failed to find track by name"));

	return *TrackPtr;
}

FTransformCurve* UAnimDataModel::FindMutableTransformCurveById(const FAnimationCurveIdentifier& CurveIdentifier)
{
	for (FTransformCurve& TransformCurve : CurveData.TransformCurves)
	{
		if (TransformCurve.GetName() == CurveIdentifier.CurveName)
		{
			return &TransformCurve;
		}
	}

	return nullptr;
}

FFloatCurve* UAnimDataModel::FindMutableFloatCurveById(const FAnimationCurveIdentifier& CurveIdentifier)
{
	for (FFloatCurve& FloatCurve : CurveData.FloatCurves)
	{
		if (FloatCurve.GetName() == CurveIdentifier.CurveName)
		{
			return &FloatCurve;
		}
	}

	return nullptr;
}

FAnimCurveBase* UAnimDataModel::FindMutableCurveById(const FAnimationCurveIdentifier& CurveIdentifier)
{
	switch (CurveIdentifier.CurveType)
	{
	case ERawCurveTrackTypes::RCT_Float:
		return FindMutableFloatCurveById(CurveIdentifier);
	case ERawCurveTrackTypes::RCT_Transform:
		return FindMutableTransformCurveById(CurveIdentifier);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE // "AnimDataModel"
