// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimEncoding.cpp: Skeletal mesh animation functions.
=============================================================================*/ 

#include "AnimEncoding.h"

// known codecs
#include "AnimEncoding_VariableKeyLerp.h"
#include "AnimEncoding_PerTrackCompression.h"

/** Each CompresedTranslationData track's ByteStream will be byte swapped in chunks of this size. */
const int32 CompressedTranslationStrides[ACF_MAX] =
{
	sizeof(float),						// ACF_None					(float X, float Y, float Z)
	sizeof(float),						// ACF_Float96NoW			(float X, float Y, float Z)
	sizeof(float),						// ACF_Fixed48NoW			(Illegal value for translation)
	sizeof(FVectorIntervalFixed32NoW),	// ACF_IntervalFixed32NoW	(compressed to 11-11-10 per-component interval fixed point)
	sizeof(float),						// ACF_Fixed32NoW			(Illegal value for translation)
	sizeof(float),						// ACF_Float32NoW			(Illegal value for translation)
	0									// ACF_Identity
};

/** Number of swapped chunks per element. */
const int32 CompressedTranslationNum[ACF_MAX] =
{
	3,	// ACF_None					(float X, float Y, float Z)
	3,	// ACF_Float96NoW			(float X, float Y, float Z)
	3,	// ACF_Fixed48NoW			(Illegal value for translation)
	1,	// ACF_IntervalFixed32NoW	(compressed to 11-11-10 per-component interval fixed point)
	3,	// ACF_Fixed32NoW			(Illegal value for translation)
	3,	// ACF_Float32NoW			(Illegal value for translation)
	0	// ACF_Identity
};

/** Each CompresedRotationData track's ByteStream will be byte swapped in chunks of this size. */
const int32 CompressedRotationStrides[ACF_MAX] =
{
	sizeof(float),						// ACF_None					(FQuats are serialized per element hence sizeof(float) rather than sizeof(FQuat).
	sizeof(float),						// ACF_Float96NoW			(FQuats with one component dropped and the remaining three uncompressed at 32bit floating point each
	sizeof(uint16),						// ACF_Fixed48NoW			(FQuats with one component dropped and the remaining three compressed to 16-16-16 fixed point.
	sizeof(FQuatIntervalFixed32NoW),	// ACF_IntervalFixed32NoW	(FQuats with one component dropped and the remaining three compressed to 11-11-10 per-component interval fixed point.
	sizeof(FQuatFixed32NoW),			// ACF_Fixed32NoW			(FQuats with one component dropped and the remaining three compressed to 11-11-10 fixed point.
	sizeof(FQuatFloat32NoW),			// ACF_Float32NoW			(FQuats with one component dropped and the remaining three compressed to 11-11-10 floating point.
	0	// ACF_Identity
};

/** Number of swapped chunks per element. */
const int32 CompressedRotationNum[ACF_MAX] =
{
	4,	// ACF_None					(FQuats are serialized per element hence sizeof(float) rather than sizeof(FQuat).
	3,	// ACF_Float96NoW			(FQuats with one component dropped and the remaining three uncompressed at 32bit floating point each
	3,	// ACF_Fixed48NoW			(FQuats with one component dropped and the remaining three compressed to 16-16-16 fixed point.
	1,	// ACF_IntervalFixed32NoW	(FQuats with one component dropped and the remaining three compressed to 11-11-10 per-component interval fixed point.
	1,	// ACF_Fixed32NoW			(FQuats with one component dropped and the remaining three compressed to 11-11-10 fixed point.
	1,  // ACF_Float32NoW			(FQuats with one component dropped and the remaining three compressed to 11-11-10 floating point.
	0	// ACF_Identity
};

/** Number of swapped chunks per element, split out per component (high 3 bits) and flags (low 3 bits)
  *
  * Note: The entry for ACF_IntervalFixed32NoW is special, and actually indicates how many fixed components there are!
  **/
const uint8 PerTrackNumComponentTable[ACF_MAX * 8] =
{
	4,4,4,4,4,4,4,4,	// ACF_None
	3,1,1,2,1,2,2,3,	// ACF_Float96NoW (0 is special, as uncompressed rotation gets 'mis'encoded with 0 instead of 7, so it's treated as a 3; a genuine 0 would use ACF_Identity)
	3,1,1,2,1,2,2,3,	// ACF_Fixed48NoW (ditto)
	6,2,2,4,2,4,4,6,	// ACF_IntervalFixed32NoW (special, indicates number of interval pairs stored in the fixed track)
	1,1,1,1,1,1,1,1,	// ACF_Fixed32NoW
	1,1,1,1,1,1,1,1,	// ACF_Float32NoW
	0,0,0,0,0,0,0,0		// ACF_Identity
};

/** Each CompresedScaleData track's ByteStream will be byte swapped in chunks of this size. */
const int32 CompressedScaleStrides[ACF_MAX] =
{
	sizeof(float),						// ACF_None					(float X, float Y, float Z)
	sizeof(float),						// ACF_Float96NoW			(float X, float Y, float Z)
	sizeof(float),						// ACF_Fixed48NoW			(Illegal value for Scale)
	sizeof(FVectorIntervalFixed32NoW),	// ACF_IntervalFixed32NoW	(compressed to 11-11-10 per-component interval fixed point)
	sizeof(float),						// ACF_Fixed32NoW			(Illegal value for Scale)
	sizeof(float),						// ACF_Float32NoW			(Illegal value for Scale)
	0									// ACF_Identity
};

/** Number of swapped chunks per element. */
const int32 CompressedScaleNum[ACF_MAX] =
{
	3,	// ACF_None					(float X, float Y, float Z)
	3,	// ACF_Float96NoW			(float X, float Y, float Z)
	3,	// ACF_Fixed48NoW			(Illegal value for Scale)
	1,	// ACF_IntervalFixed32NoW	(compressed to 11-11-10 per-component interval fixed point)
	3,	// ACF_Fixed32NoW			(Illegal value for Scale)
	3,	// ACF_Float32NoW			(Illegal value for Scale)
	0	// ACF_Identity
};
/**
 * Compressed translation data will be byte swapped in chunks of this size.
 */
inline int32 GetCompressedTranslationStride(AnimationCompressionFormat TranslationCompressionFormat)
{
	return CompressedTranslationStrides[TranslationCompressionFormat];
}

/**
 * Compressed rotation data will be byte swapped in chunks of this size.
 */
inline int32 GetCompressedRotationStride(AnimationCompressionFormat RotationCompressionFormat)
{
	return CompressedRotationStrides[RotationCompressionFormat];
}

/**
 * Compressed Scale data will be byte swapped in chunks of this size.
 */
inline int32 GetCompressedScaleStride(AnimationCompressionFormat ScaleCompressionFormat)
{
	// @todo fixme change this?
	return CompressedScaleStrides[ScaleCompressionFormat];
}

/**
 * Compressed translation data will be byte swapped in chunks of this size.
 */
inline int32 GetCompressedTranslationStride(const FUECompressedAnimData& CompressedData)
{
	return CompressedTranslationStrides[static_cast<AnimationCompressionFormat>(CompressedData.TranslationCompressionFormat)];
}

/**
 * Compressed rotation data will be byte swapped in chunks of this size.
 */
inline int32 GetCompressedRotationStride(const FUECompressedAnimData& CompressedData)
{
	return CompressedRotationStrides[static_cast<AnimationCompressionFormat>(CompressedData.RotationCompressionFormat)];
}

/**
 * Compressed Scale data will be byte swapped in chunks of this size.
 */
inline int32 GetCompressedScaleStride(const FUECompressedAnimData& CompressedData)
{
	// @todo fixme change this?
	return CompressedScaleStrides[static_cast<AnimationCompressionFormat>(CompressedData.ScaleCompressionFormat)];
}


/**
 * Pads a specified number of bytes to the memory writer to maintain alignment
 */
void PadMemoryWriter(FMemoryWriter* MemoryWriter, uint8*& TrackData, const int32 Alignment)
{
	const PTRINT ByteStreamLoc = (PTRINT) TrackData;
	const int32 Pad = static_cast<int32>( Align( ByteStreamLoc, Alignment ) - ByteStreamLoc );
	const uint8 PadSentinel = 85; // (1<<1)+(1<<3)+(1<<5)+(1<<7)
	
	for ( int32 PadByteIndex = 0; PadByteIndex < Pad; ++PadByteIndex )
	{
		MemoryWriter->Serialize( (void*)&PadSentinel, sizeof(uint8) );
	}
	TrackData += Pad;
}

/**
 * Skips a specified number of bytes in the memory reader to maintain alignment
 */
void PadMemoryReader(FMemoryReader* MemoryReader, uint8*& TrackData, const int32 Alignment)
{
	const PTRINT ByteStreamLoc = (PTRINT) TrackData;
	const int32 Pad = static_cast<int32>( Align( ByteStreamLoc, Alignment ) - ByteStreamLoc );
	MemoryReader->Serialize( TrackData, Pad );
	TrackData += Pad;
}

/**
 * Handles Byte-swapping incoming animation data from a MemoryReader
 *
 * @param	CompressedData		The compressed animation data being operated on.
 * @param	MemoryReader		The MemoryReader object to read from.
 */
void AnimEncodingLegacyBase::ByteSwapIn(
	FUECompressedAnimData& CompressedData,
	FMemoryReader& MemoryReader)
{
	const int32 BufferStart = MemoryReader.Tell();

	const int32 NumTracks = CompressedData.CompressedTrackOffsets.Num() / 4;

	int32 OriginalNumBytes = MemoryReader.TotalSize() - BufferStart;
	check(CompressedData.CompressedByteStream.Num() == OriginalNumBytes);

	// Read and swap
	uint8* StreamBase = CompressedData.CompressedByteStream.GetData();
	bool bHasValidScale = CompressedData.CompressedScaleOffsets.IsValid();

	for ( int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex )
	{
		const int32 OffsetTrans	= CompressedData.CompressedTrackOffsets[TrackIndex*4+0];
		const int32 NumKeysTrans	= CompressedData.CompressedTrackOffsets[TrackIndex*4+1];
		const int32 OffsetRot		= CompressedData.CompressedTrackOffsets[TrackIndex*4+2];
		const int32 NumKeysRot	= CompressedData.CompressedTrackOffsets[TrackIndex*4+3];

		// Translation data.
		checkSlow( (OffsetTrans % 4) == 0 && "CompressedByteStream not aligned to four bytes" );
		uint8* TransTrackData = StreamBase + OffsetTrans;
		checkSlow(CompressedData.TranslationCodec != NULL);
		((AnimEncodingLegacyBase*)CompressedData.TranslationCodec)->ByteSwapTranslationIn(CompressedData, MemoryReader, TransTrackData, NumKeysTrans);

		// Like the compressed byte stream, pad the serialization stream to four bytes.
		// As a sanity check, each pad byte can be checked to be the PadSentinel.
		PadMemoryReader(&MemoryReader, TransTrackData, 4); 

		// Rotation data.
		checkSlow( (OffsetRot % 4) == 0 && "CompressedByteStream not aligned to four bytes" );
		uint8* RotTrackData = StreamBase + OffsetRot;
		checkSlow(CompressedData.RotationCodec != NULL);
		((AnimEncodingLegacyBase*)CompressedData.RotationCodec)->ByteSwapRotationIn(CompressedData, MemoryReader, RotTrackData, NumKeysRot);

		// Like the compressed byte stream, pad the serialization stream to four bytes.
		// As a sanity check, each pad byte can be checked to be the PadSentinel.
		PadMemoryReader(&MemoryReader, RotTrackData, 4); 

		if (bHasValidScale)
		{
			const int32 OffsetScale		= CompressedData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 0);
			const int32 NumKeysScale		= CompressedData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 1);

			// Scale data.
			checkSlow( (OffsetScale % 4) == 0 && "CompressedByteStream not aligned to four bytes" );
			uint8* ScaleTrackData = StreamBase + OffsetScale;
			checkSlow(CompressedData.ScaleCodec != NULL);
			((AnimEncodingLegacyBase*)CompressedData.ScaleCodec)->ByteSwapScaleIn(CompressedData, MemoryReader, ScaleTrackData, NumKeysScale);

			// Like the compressed byte stream, pad the serialization stream to four bytes.
			// As a sanity check, each pad byte can be checked to be the PadSentinel.
			PadMemoryReader(&MemoryReader, ScaleTrackData, 4); 
		}
	}
}

/**
 * Handles Byte-swapping outgoing animation data to an array of BYTEs
 *
 * @param	Seq					An Animation Sequence to write.
 * @param	SerializedData		The output buffer.
 * @param	ForceByteSwapping	true is byte swapping is not optional.
 */
void AnimEncodingLegacyBase::ByteSwapOut(
	FUECompressedAnimData& CompressedData,
	FMemoryWriter& MemoryWriter)
{
	const int32 BufferStart = MemoryWriter.Tell();

	uint8* StreamBase		= CompressedData.CompressedByteStream.GetData();
	const int32 NumTracks		= CompressedData.CompressedTrackOffsets.Num()/4;

	bool bHasValidScale = CompressedData.CompressedScaleOffsets.IsValid();

	for ( int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex )
	{
		const int32 OffsetTrans	= CompressedData.CompressedTrackOffsets[TrackIndex*4];
		const int32 NumKeysTrans	= CompressedData.CompressedTrackOffsets[TrackIndex*4+1];
		const int32 OffsetRot		= CompressedData.CompressedTrackOffsets[TrackIndex*4+2];
		const int32 NumKeysRot	= CompressedData.CompressedTrackOffsets[TrackIndex*4+3];

		MemoryWriter.Seek(BufferStart + OffsetTrans);

		// Translation data.
		checkSlow( (OffsetTrans % 4) == 0 && "CompressedByteStream not aligned to four bytes" );
		uint8* TransTrackData = StreamBase + OffsetTrans;
		if (CompressedData.TranslationCodec != NULL)
		{
			((AnimEncodingLegacyBase*)CompressedData.TranslationCodec)->ByteSwapTranslationOut(CompressedData, MemoryWriter, TransTrackData, NumKeysTrans);
		}
		else
		{
			UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported animation format", (int32)CompressedData.KeyEncodingFormat );
		};

		// Like the compressed byte stream, pad the serialization stream to four bytes.
		PadMemoryWriter(&MemoryWriter, TransTrackData, 4);

		MemoryWriter.Seek(BufferStart + OffsetRot);

		// Rotation data.
		checkSlow( (OffsetRot % 4) == 0 && "CompressedByteStream not aligned to four bytes" );
		uint8* RotTrackData = StreamBase + OffsetRot;
		checkSlow(CompressedData.RotationCodec != NULL);
		((AnimEncodingLegacyBase*)CompressedData.RotationCodec)->ByteSwapRotationOut(CompressedData, MemoryWriter, RotTrackData, NumKeysRot);

		// Like the compressed byte stream, pad the serialization stream to four bytes.
		PadMemoryWriter(&MemoryWriter, RotTrackData, 4);

		if (bHasValidScale)
		{
			const int32 OffsetScale	= CompressedData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 0);
			const int32 NumKeysScale	= CompressedData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 1);

			MemoryWriter.Seek(BufferStart + OffsetScale);

			// Scale data.
			checkSlow( (OffsetScale % 4) == 0 && "CompressedByteStream not aligned to four bytes" );
			uint8* ScaleTrackData = StreamBase + OffsetScale;
			if (CompressedData.ScaleCodec != NULL)
			{
				((AnimEncodingLegacyBase*)CompressedData.ScaleCodec)->ByteSwapScaleOut(CompressedData, MemoryWriter, ScaleTrackData, NumKeysScale);
			}
			else
			{
				UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported animation format", (int32)CompressedData.KeyEncodingFormat );
			};

			// Like the compressed byte stream, pad the serialization stream to four bytes.
			PadMemoryWriter(&MemoryWriter, ScaleTrackData, 4);
		}
	}
}

/**
 * Extracts statistics about a given Animation Sequence
 *
 * @param	Seq					An Animation Sequence.
 * @param	NumTransTracks		The total number of Translation Tracks found.
 * @param	NumRotTracks		The total number of Rotation Tracks found.
 * @param	TotalNumTransKeys	The total number of Translation Keys found.
 * @param	TotalNumRotKeys		The total number of Rotation Keys found.
 * @param	TranslationKeySize	The average size (in BYTES) of a single Translation Key.
 * @param	RotationKeySize		The average size (in BYTES) of a single Rotation Key.
 * @param	OverheadSize		The size (in BYTES) of overhead (offsets, scale tables, key->frame lookups, etc...)
 * @param	NumTransTracksWithOneKey	The total number of Translation Tracks found containing a single key.
 * @param	NumRotTracksWithOneKey		The total number of Rotation Tracks found containing a single key.
*/
void AnimationFormat_GetStats(	
	const FUECompressedAnimData& CompressedData, 
	int32& NumTransTracks,
	int32& NumRotTracks,
	int32& NumScaleTracks,
	int32& TotalNumTransKeys,
	int32& TotalNumRotKeys,
	int32& TotalNumScaleKeys,
	float& TranslationKeySize,
	float& RotationKeySize,
	float& ScaleKeySize,
	int32& OverheadSize,
	int32& NumTransTracksWithOneKey,
	int32& NumRotTracksWithOneKey, 
	int32& NumScaleTracksWithOneKey)
{
	OverheadSize = CompressedData.CompressedTrackOffsets.Num() * sizeof(int32);
	const size_t KeyFrameLookupSize = (CompressedData.CompressedNumberOfKeys > 0xFF) ? sizeof(uint16) : sizeof(uint8);

	if (CompressedData.KeyEncodingFormat != AKF_PerTrackCompression)
	{
		const int32 TransStride	= GetCompressedTranslationStride(CompressedData);
		const int32 RotStride		= GetCompressedRotationStride(CompressedData);
		const int32 ScaleStride		= GetCompressedScaleStride(CompressedData);
		const int32 TransNum		= CompressedTranslationNum[CompressedData.TranslationCompressionFormat];
		const int32 RotNum		= CompressedRotationNum[CompressedData.RotationCompressionFormat];
		const int32 ScaleNum		= CompressedScaleNum[CompressedData.ScaleCompressionFormat];

		TranslationKeySize = TransStride * TransNum;
		RotationKeySize = RotStride * RotNum;
		ScaleKeySize = ScaleStride * ScaleNum;

		// Track number of tracks.
		NumTransTracks	= CompressedData.CompressedTrackOffsets.Num()/4;
		NumRotTracks	= CompressedData.CompressedTrackOffsets.Num()/4;
		NumScaleTracks	= CompressedData.CompressedScaleOffsets.GetNumTracks();

		// Track total number of keys.
		TotalNumTransKeys = 0;
		TotalNumRotKeys = 0;
		TotalNumScaleKeys = 0;

		// Track number of tracks with a single key.
		NumTransTracksWithOneKey = 0;
		NumRotTracksWithOneKey = 0;
		NumScaleTracksWithOneKey = 0;

		// Translation.
		for ( int32 TrackIndex = 0; TrackIndex < NumTransTracks; ++TrackIndex )
		{
			const int32 NumKeys = CompressedData.CompressedTrackOffsets[TrackIndex*4+1];
			TotalNumTransKeys += NumKeys;
			if ( NumKeys == 1 )
			{
				++NumTransTracksWithOneKey;
			}
			else
			{
				OverheadSize += (CompressedData.KeyEncodingFormat == AKF_VariableKeyLerp) ? NumKeys * KeyFrameLookupSize : 0;
			}
		}

		// Rotation.
		for ( int32 TrackIndex = 0; TrackIndex < NumRotTracks; ++TrackIndex )
		{
			const int32 NumKeys = CompressedData.CompressedTrackOffsets[TrackIndex*4+3];
			TotalNumRotKeys += NumKeys;
			if ( NumKeys == 1 )
			{
				++NumRotTracksWithOneKey;
			}
			else
			{
				OverheadSize += (CompressedData.KeyEncodingFormat == AKF_VariableKeyLerp) ? NumKeys * KeyFrameLookupSize : 0;
			}
		}

		// Scale.
		for ( int32 ScaleIndex = 0; ScaleIndex < NumScaleTracks; ++ScaleIndex )
		{
			const int32 NumKeys = CompressedData.CompressedScaleOffsets.GetOffsetData(ScaleIndex, 1);
			TotalNumScaleKeys += NumKeys;
			if ( NumKeys == 1 )
			{
				++NumScaleTracksWithOneKey;
			}
			else
			{
				OverheadSize += (CompressedData.KeyEncodingFormat == AKF_VariableKeyLerp) ? NumKeys * KeyFrameLookupSize : 0;
			}
		}

		// Add in scaling values (min+range for interval encoding)
		OverheadSize += (CompressedData.RotationCompressionFormat == ACF_IntervalFixed32NoW) ? (NumRotTracks - NumRotTracksWithOneKey) * sizeof(float) * 6 : 0;
		OverheadSize += (CompressedData.TranslationCompressionFormat == ACF_IntervalFixed32NoW) ? (NumTransTracks - NumTransTracksWithOneKey) * sizeof(float) * 6 : 0;
		OverheadSize += (CompressedData.ScaleCompressionFormat == ACF_IntervalFixed32NoW) ? (NumScaleTracks - NumScaleTracksWithOneKey) * sizeof(float) * 6 : 0;
	}
	else
	{
		int32 TotalTransKeysThatContributedSize = 0;
		int32 TotalRotKeysThatContributedSize = 0;
		int32 TotalScaleKeysThatContributedSize = 0;

		TranslationKeySize = 0;
		RotationKeySize = 0;
		ScaleKeySize = 0;

		// Track number of tracks.
		NumTransTracks = CompressedData.CompressedTrackOffsets.Num() / 2;
		NumRotTracks = CompressedData.CompressedTrackOffsets.Num() / 2;
		// should be without divided by 2
		NumScaleTracks = CompressedData.CompressedScaleOffsets.GetNumTracks();

		// Track total number of keys.
		TotalNumTransKeys = 0;
		TotalNumRotKeys = 0;
		TotalNumScaleKeys = 0;

		// Track number of tracks with a single key.
		NumTransTracksWithOneKey = 0;
		NumRotTracksWithOneKey = 0;
		NumScaleTracksWithOneKey = 0;

		// Translation.
		for (int32 TrackIndex = 0; TrackIndex < NumTransTracks; ++TrackIndex)
		{
			const int32 TransOffset = CompressedData.CompressedTrackOffsets[TrackIndex*2+0];
			if (TransOffset == INDEX_NONE)
			{
				++TotalNumTransKeys;
				++NumTransTracksWithOneKey;
			}
			else
			{
				const int32 Header = *((int32*)(CompressedData.CompressedByteStream.GetData() + TransOffset));

				int32 KeyFormat;
				int32 FormatFlags;
				int32 TrackBytesPerKey;
				int32 TrackFixedBytes;
				int32 NumKeys;

				FAnimationCompression_PerTrackUtils::DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/ TrackBytesPerKey, /*OUT*/ TrackFixedBytes);
				TranslationKeySize += TrackBytesPerKey * NumKeys;
				TotalTransKeysThatContributedSize += NumKeys;
				OverheadSize += TrackFixedBytes;
				OverheadSize += ((FormatFlags & 0x08) != 0) ? NumKeys * KeyFrameLookupSize : 0;
					
				TotalNumTransKeys += NumKeys;
				if (NumKeys <= 1)
				{
					++NumTransTracksWithOneKey;
				}
			}
		}

		// Rotation.
		for (int32 TrackIndex = 0; TrackIndex < NumRotTracks; ++TrackIndex)
		{
			const int32 RotOffset = CompressedData.CompressedTrackOffsets[TrackIndex*2+1];
			if (RotOffset == INDEX_NONE)
			{
				++TotalNumRotKeys;
				++NumRotTracksWithOneKey;
			}
			else
			{
				const int32 Header = *((int32*)(CompressedData.CompressedByteStream.GetData() + RotOffset));

				int32 KeyFormat;
				int32 FormatFlags;
				int32 TrackBytesPerKey;
				int32 TrackFixedBytes;
				int32 NumKeys;

				FAnimationCompression_PerTrackUtils::DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/ TrackBytesPerKey, /*OUT*/ TrackFixedBytes);
				RotationKeySize += TrackBytesPerKey * NumKeys;
				TotalRotKeysThatContributedSize += NumKeys;
				OverheadSize += TrackFixedBytes;
				OverheadSize += ((FormatFlags & 0x08) != 0) ? NumKeys * KeyFrameLookupSize : 0;

				TotalNumRotKeys += NumKeys;
				if (NumKeys <= 1)
				{
					++NumRotTracksWithOneKey;
				}
			}
		}

		// Scale.
		for (int32 TrackIndex = 0; TrackIndex < NumScaleTracks; ++TrackIndex)
		{
			const int32 ScaleOffset = CompressedData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 0);
			if (ScaleOffset == INDEX_NONE)
			{
				++TotalNumScaleKeys;
				++NumScaleTracksWithOneKey;
			}
			else
			{
				const int32 Header = *((int32*)(CompressedData.CompressedByteStream.GetData() + ScaleOffset));

				int32 KeyFormat;
				int32 FormatFlags;
				int32 TrackBytesPerKey;
				int32 TrackFixedBytes;
				int32 NumKeys;

				FAnimationCompression_PerTrackUtils::DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/ TrackBytesPerKey, /*OUT*/ TrackFixedBytes);
				ScaleKeySize += TrackBytesPerKey * NumKeys;
				TotalScaleKeysThatContributedSize += NumKeys;
				OverheadSize += TrackFixedBytes;
				OverheadSize += ((FormatFlags & 0x08) != 0) ? NumKeys * KeyFrameLookupSize : 0;

				TotalNumScaleKeys += NumKeys;
				if (NumKeys <= 1)
				{
					++NumScaleTracksWithOneKey;
				}
			}
		}

		// Average key sizes
		if (TotalRotKeysThatContributedSize > 0)
		{
			RotationKeySize = RotationKeySize / TotalRotKeysThatContributedSize;
		}

		if (TotalTransKeysThatContributedSize > 0)
		{
			TranslationKeySize = TranslationKeySize / TotalTransKeysThatContributedSize;
		}

		if (TotalScaleKeysThatContributedSize > 0)
		{
			ScaleKeySize = ScaleKeySize / TotalScaleKeysThatContributedSize;
		}
	}
}

/**
 * Sets the internal Animation Codec Interface Links within an Animation CompressedDatauence
 *
 * @param	Seq					An Animation Sequence to setup links within.
*/
template<typename CompressedDataType>
void AnimationFormat_SetInterfaceLinks(CompressedDataType& CompressedData)
{
	CompressedData.TranslationCodec = NULL;
	CompressedData.RotationCodec = NULL;
	CompressedData.ScaleCodec = NULL;

	if (CompressedData.KeyEncodingFormat == AKF_ConstantKeyLerp)
	{
		static AEFConstantKeyLerp<ACF_None>					AEFConstantKeyLerp_None;
		static AEFConstantKeyLerp<ACF_Float96NoW>			AEFConstantKeyLerp_Float96NoW;
		static AEFConstantKeyLerp<ACF_Fixed48NoW>			AEFConstantKeyLerp_Fixed48NoW;
		static AEFConstantKeyLerp<ACF_IntervalFixed32NoW>	AEFConstantKeyLerp_IntervalFixed32NoW;
		static AEFConstantKeyLerp<ACF_Fixed32NoW>			AEFConstantKeyLerp_Fixed32NoW;
		static AEFConstantKeyLerp<ACF_Float32NoW>			AEFConstantKeyLerp_Float32NoW;
		static AEFConstantKeyLerp<ACF_Identity>				AEFConstantKeyLerp_Identity;

		// setup translation codec
		switch(CompressedData.TranslationCompressionFormat)
		{
			case ACF_None:
				CompressedData.TranslationCodec = &AEFConstantKeyLerp_None;
				break;
			case ACF_Float96NoW:
				CompressedData.TranslationCodec = &AEFConstantKeyLerp_Float96NoW;
				break;
			case ACF_IntervalFixed32NoW:
				CompressedData.TranslationCodec = &AEFConstantKeyLerp_IntervalFixed32NoW;
				break;
			case ACF_Identity:
				CompressedData.TranslationCodec = &AEFConstantKeyLerp_Identity;
				break;

			default:
				UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported translation compression", (int32)CompressedData.TranslationCompressionFormat );
		};

		// setup rotation codec
		switch(CompressedData.RotationCompressionFormat)
		{
			case ACF_None:
				CompressedData.RotationCodec = &AEFConstantKeyLerp_None;
				break;
			case ACF_Float96NoW:
				CompressedData.RotationCodec = &AEFConstantKeyLerp_Float96NoW;
				break;
			case ACF_Fixed48NoW:
				CompressedData.RotationCodec = &AEFConstantKeyLerp_Fixed48NoW;
				break;
			case ACF_IntervalFixed32NoW:
				CompressedData.RotationCodec = &AEFConstantKeyLerp_IntervalFixed32NoW;
				break;
			case ACF_Fixed32NoW:
				CompressedData.RotationCodec = &AEFConstantKeyLerp_Fixed32NoW;
				break;
			case ACF_Float32NoW:
				CompressedData.RotationCodec = &AEFConstantKeyLerp_Float32NoW;
				break;
			case ACF_Identity:
				CompressedData.RotationCodec = &AEFConstantKeyLerp_Identity;
				break;

			default:
				UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported rotation compression", (int32)CompressedData.RotationCompressionFormat );
		};

		// setup Scale codec
		switch(CompressedData.ScaleCompressionFormat)
		{
		case ACF_None:
			CompressedData.ScaleCodec = &AEFConstantKeyLerp_None;
			break;
		case ACF_Float96NoW:
			CompressedData.ScaleCodec = &AEFConstantKeyLerp_Float96NoW;
			break;
		case ACF_IntervalFixed32NoW:
			CompressedData.ScaleCodec = &AEFConstantKeyLerp_IntervalFixed32NoW;
			break;
		case ACF_Identity:
			CompressedData.ScaleCodec = &AEFConstantKeyLerp_Identity;
			break;

		default:
			UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported Scale compression", (int32)CompressedData.ScaleCompressionFormat );
		};
	}
	else if (CompressedData.KeyEncodingFormat == AKF_VariableKeyLerp)
	{
		static AEFVariableKeyLerp<ACF_None>					AEFVariableKeyLerp_None;
		static AEFVariableKeyLerp<ACF_Float96NoW>			AEFVariableKeyLerp_Float96NoW;
		static AEFVariableKeyLerp<ACF_Fixed48NoW>			AEFVariableKeyLerp_Fixed48NoW;
		static AEFVariableKeyLerp<ACF_IntervalFixed32NoW>	AEFVariableKeyLerp_IntervalFixed32NoW;
		static AEFVariableKeyLerp<ACF_Fixed32NoW>			AEFVariableKeyLerp_Fixed32NoW;
		static AEFVariableKeyLerp<ACF_Float32NoW>			AEFVariableKeyLerp_Float32NoW;
		static AEFVariableKeyLerp<ACF_Identity>				AEFVariableKeyLerp_Identity;

		// setup translation codec
		switch(CompressedData.TranslationCompressionFormat)
		{
			case ACF_None:
				CompressedData.TranslationCodec = &AEFVariableKeyLerp_None;
				break;
			case ACF_Float96NoW:
				CompressedData.TranslationCodec = &AEFVariableKeyLerp_Float96NoW;
				break;
			case ACF_IntervalFixed32NoW:
				CompressedData.TranslationCodec = &AEFVariableKeyLerp_IntervalFixed32NoW;
				break;
			case ACF_Identity:
				CompressedData.TranslationCodec = &AEFVariableKeyLerp_Identity;
				break;

			default:
				UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported translation compression", (int32)CompressedData.TranslationCompressionFormat );
		};

		// setup rotation codec
		switch(CompressedData.RotationCompressionFormat)
		{
			case ACF_None:
				CompressedData.RotationCodec = &AEFVariableKeyLerp_None;
				break;
			case ACF_Float96NoW:
				CompressedData.RotationCodec = &AEFVariableKeyLerp_Float96NoW;
				break;
			case ACF_Fixed48NoW:
				CompressedData.RotationCodec = &AEFVariableKeyLerp_Fixed48NoW;
				break;
			case ACF_IntervalFixed32NoW:
				CompressedData.RotationCodec = &AEFVariableKeyLerp_IntervalFixed32NoW;
				break;
			case ACF_Fixed32NoW:
				CompressedData.RotationCodec = &AEFVariableKeyLerp_Fixed32NoW;
				break;
			case ACF_Float32NoW:
				CompressedData.RotationCodec = &AEFVariableKeyLerp_Float32NoW;
				break;
			case ACF_Identity:
				CompressedData.RotationCodec = &AEFVariableKeyLerp_Identity;
				break;

			default:
				UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported rotation compression", (int32)CompressedData.RotationCompressionFormat );
		};

		// setup Scale codec
		switch(CompressedData.ScaleCompressionFormat)
		{
		case ACF_None:
			CompressedData.ScaleCodec = &AEFVariableKeyLerp_None;
			break;
		case ACF_Float96NoW:
			CompressedData.ScaleCodec = &AEFVariableKeyLerp_Float96NoW;
			break;
		case ACF_IntervalFixed32NoW:
			CompressedData.ScaleCodec = &AEFVariableKeyLerp_IntervalFixed32NoW;
			break;
		case ACF_Identity:
			CompressedData.ScaleCodec = &AEFVariableKeyLerp_Identity;
			break;

		default:
			UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported Scale compression", (int32)CompressedData.ScaleCompressionFormat );
		};
	}
	else if (CompressedData.KeyEncodingFormat == AKF_PerTrackCompression)
	{
		static AEFPerTrackCompressionCodec StaticCodec;

		CompressedData.RotationCodec = &StaticCodec;
		CompressedData.TranslationCodec = &StaticCodec;
		CompressedData.ScaleCodec = &StaticCodec;

		check(CompressedData.RotationCompressionFormat == ACF_Identity);
		check(CompressedData.TranslationCompressionFormat == ACF_Identity);
		// commenting out scale check because the older versions won't have this set correctly
		// and I can't get the version VER_UE4_ANIM_SUPPORT_NONUNIFORM_SCALE_ANIMATION here because this function
		// is called in Serialize where GetLinker is too early to call
		//checkf(CompressedData.ScaleCompressionFormat == ACF_Identity);
	}
	else
	{
		UE_LOGF(LogAnimationCompression, Fatal, "%i: unknown or unsupported animation format", (int32)CompressedData.KeyEncodingFormat );
	}
}

template void AnimationFormat_SetInterfaceLinks(FUECompressedAnimData& CompressedData);
template void AnimationFormat_SetInterfaceLinks(FUECompressedAnimDataMutable& CompressedData);
