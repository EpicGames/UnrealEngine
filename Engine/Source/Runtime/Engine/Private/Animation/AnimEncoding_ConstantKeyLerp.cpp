// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimEncoding_ConstantKeyLerp.cpp: Skeletal mesh animation functions.
=============================================================================*/ 

#include "AnimEncoding_ConstantKeyLerp.h"
#if INTEL_ISPC
#include "AnimEncoding_ConstantKeyLerp.ispc.generated.h"
#include "AutoRTFM.h"

static_assert(sizeof(ispc::FTransform) == sizeof(FTransform), "sizeof(ispc::FTransform) != sizeof(FTransform)");
static_assert(sizeof(ispc::BoneTrackPair) == sizeof(BoneTrackPair), "sizeof(ispc::BoneTrackPair) != sizeof(BoneTrackPair)");
#endif

#if !defined(ANIM_CONSTANT_KEY_LERP_ISPC_ENABLED_DEFAULT)
#define ANIM_CONSTANT_KEY_LERP_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bAnim_ConstantKeyLerp_ISPC_Enabled = INTEL_ISPC && ANIM_CONSTANT_KEY_LERP_ISPC_ENABLED_DEFAULT;
#else
#include "HAL/IConsoleManager.h"
static bool bAnim_ConstantKeyLerp_ISPC_Enabled = ANIM_CONSTANT_KEY_LERP_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarAnimConstantKeyLerpISPCEnabled(TEXT("a.ConstantKeyLerp.ISPC"), bAnim_ConstantKeyLerp_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in constant key anim encoding"));
#endif

/**
 * Handles the ByteSwap of compressed rotation data on import
 *
 * @param	CompressedData		The compressed animation data being operated on.
 * @param	MemoryReader	The FMemoryReader to read from.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys present in the stream.
 */
void AEFConstantKeyLerpShared::ByteSwapRotationIn(
	FUECompressedAnimData& CompressedData,
	FMemoryReader& MemoryReader,
	uint8*& TrackData,
	int32 NumKeys)
{
	// Calculate the effective compression (in a track with only one key, it's always stored lossless)
	const int32 EffectiveFormat = (NumKeys == 1) ? ACF_Float96NoW : (int32)CompressedData.RotationCompressionFormat;
	const int32 KeyComponentSize = CompressedRotationStrides[EffectiveFormat];
	const int32 KeyNumComponents = CompressedRotationNum[EffectiveFormat];

	// Load the bounds if present
	if (EffectiveFormat == ACF_IntervalFixed32NoW)
	{
		for (int32 i = 0; i < 6; ++i)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, sizeof(float));
		}
	}

	// Load the keys
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		for (int32 i = 0; i < KeyNumComponents; ++i)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, KeyComponentSize);
		}
	}
}

/**
 * Handles the ByteSwap of compressed translation data on import
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryReader	The FMemoryReader to read from.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys present in the stream.
 */
void AEFConstantKeyLerpShared::ByteSwapTranslationIn(
	FUECompressedAnimData& CompressedData,
	FMemoryReader& MemoryReader,
	uint8*& TrackData,
	int32 NumKeys)
{
	// Calculate the effective compression (in a track with only one key, it's always stored lossless)
	const int32 EffectiveFormat = (NumKeys == 1) ? ACF_None : (int32) CompressedData.TranslationCompressionFormat;
	const int32 KeyComponentSize = CompressedTranslationStrides[EffectiveFormat];
	const int32 KeyNumComponents = CompressedTranslationNum[EffectiveFormat];

	// Load the bounds if present
	if (EffectiveFormat == ACF_IntervalFixed32NoW)
	{
		for (int32 i = 0; i < 6; ++i)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, sizeof(float));
		}
	}

	// Load the keys
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		for (int32 i = 0; i < KeyNumComponents; ++i)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, KeyComponentSize);
		}
	}
}

/**
 * Handles the ByteSwap of compressed Scale data on import
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryReader	The FMemoryReader to read from.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys present in the stream.
 */
void AEFConstantKeyLerpShared::ByteSwapScaleIn(
	FUECompressedAnimData& CompressedData,
	FMemoryReader& MemoryReader,
	uint8*& TrackData,
	int32 NumKeys)
{
	// Calculate the effective compression (in a track with only one key, it's always stored lossless)
	const int32 EffectiveFormat = (NumKeys == 1) ? ACF_None : (int32)CompressedData.ScaleCompressionFormat;
	const int32 KeyComponentSize = CompressedScaleStrides[EffectiveFormat];
	const int32 KeyNumComponents = CompressedScaleNum[EffectiveFormat];

	// Load the bounds if present
	if (EffectiveFormat == ACF_IntervalFixed32NoW)
	{
		for (int32 i = 0; i < 6; ++i)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, sizeof(float));
		}
	}

	// Load the keys
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		for (int32 i = 0; i < KeyNumComponents; ++i)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, KeyComponentSize);
		}
	}
}
/**
 * Handles the ByteSwap of compressed rotation data on export
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryWriter	The FMemoryWriter to write to.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys to write to the stream.
 */
void AEFConstantKeyLerpShared::ByteSwapRotationOut(
	FUECompressedAnimData& CompressedData,
	FMemoryWriter& MemoryWriter,
	uint8*& TrackData,
	int32 NumKeys)
{
	// Calculate the effective compression (in a track with only one key, it's always stored lossless)
	const int32 EffectiveFormat = (NumKeys == 1) ? ACF_Float96NoW : (int32) CompressedData.RotationCompressionFormat;
	const int32 KeyComponentSize = CompressedRotationStrides[EffectiveFormat];
	const int32 KeyNumComponents = CompressedRotationNum[EffectiveFormat];

	// Store the bounds if needed
	if (EffectiveFormat == ACF_IntervalFixed32NoW)
	{
		for (int32 i = 0; i < 6; ++i)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, sizeof(float));
		}
	}

	// Store the keys
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		for (int32 i = 0; i < KeyNumComponents; ++i)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, KeyComponentSize);
		}
	}
}

/**
 * Handles the ByteSwap of compressed translation data on export
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryWriter	The FMemoryWriter to write to.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys to write to the stream.
 */
void AEFConstantKeyLerpShared::ByteSwapTranslationOut(
	FUECompressedAnimData& CompressedData,
	FMemoryWriter& MemoryWriter,
	uint8*& TrackData,
	int32 NumKeys)
{
	// Calculate the effective compression (in a track with only one key, it's always stored lossless)
	const int32 EffectiveFormat = (NumKeys == 1) ? ACF_None : (int32) CompressedData.TranslationCompressionFormat;
	const int32 KeyComponentSize = CompressedTranslationStrides[EffectiveFormat];
	const int32 KeyNumComponents = CompressedTranslationNum[EffectiveFormat];

	// Store the bounds if needed
	if (EffectiveFormat == ACF_IntervalFixed32NoW)
	{
		for (int32 i = 0; i < 6; ++i)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, sizeof(float));
		}
	}

	// Store the keys
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		for (int32 i = 0; i < KeyNumComponents; ++i)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, KeyComponentSize);
		}
	}
}

/**
 * Handles the ByteSwap of compressed Scale data on export
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryWriter	The FMemoryWriter to write to.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys to write to the stream.
 */
void AEFConstantKeyLerpShared::ByteSwapScaleOut(
	FUECompressedAnimData& CompressedData,
	FMemoryWriter& MemoryWriter,
	uint8*& TrackData,
	int32 NumKeys)
{
	// Calculate the effective compression (in a track with only one key, it's always stored lossless)
	const int32 EffectiveFormat = (NumKeys == 1) ? ACF_None : (int32) CompressedData.ScaleCompressionFormat;
	const int32 KeyComponentSize = CompressedScaleStrides[EffectiveFormat];
	const int32 KeyNumComponents = CompressedScaleNum[EffectiveFormat];

	// Store the bounds if needed
	if (EffectiveFormat == ACF_IntervalFixed32NoW)
	{
		for (int32 i = 0; i < 6; ++i)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, sizeof(float));
		}
	}

	// Store the keys
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		for (int32 i = 0; i < KeyNumComponents; ++i)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, KeyComponentSize);
		}
	}
}

#if USE_ANIMATION_CODEC_BATCH_SOLVER

/**
 * Decompress all requested rotation components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
template<int32 FORMAT>
inline void AEFConstantKeyLerp<FORMAT>::GetPoseRotations(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();

	if (PairCount == 0)
	{
		return;
	}

	if (bAnim_ConstantKeyLerp_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetConstantKeyLerpPoseRotations(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				AnimData.CompressedTrackOffsets.GetData(),
				AnimData.CompressedByteStream.GetData(),
				DecompContext.GetPlayableLength(),
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				FORMAT,
				PairCount);
		};
#endif
	}
	else
	{
		for (int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex)
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];

			// call the decoder directly (not through the vtable)
			AEFConstantKeyLerp<FORMAT>::GetBoneAtomRotation(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

/**
 * Decompress all requested translation components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
template<int32 FORMAT>
inline void AEFConstantKeyLerp<FORMAT>::GetPoseTranslations(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();

	if (PairCount == 0)
	{
		return;
	}

	if (bAnim_ConstantKeyLerp_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetConstantKeyLerpPoseTranslations(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				AnimData.CompressedTrackOffsets.GetData(),
				AnimData.CompressedByteStream.GetData(),
				DecompContext.GetPlayableLength(),
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				FORMAT,
				PairCount);
		};
#endif
	}
	else
	{
		//@TODO: Verify that this prefetch is helping
		// Prefetch the desired pairs array and 2 destination spots; the loop will prefetch one 2 out each iteration
		FPlatformMisc::Prefetch(&(DesiredPairs[0]));
		const int32 PrefetchCount = FMath::Min(PairCount, 1);
		for (int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex)
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			FPlatformMisc::Prefetch(Atoms.GetData() + Pair.AtomIndex);
		}

		for (int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex)
		{
			int32 PrefetchIndex = PairIndex + PrefetchCount;
			if (PrefetchIndex < PairCount)
			{
				FPlatformMisc::Prefetch(Atoms.GetData() + DesiredPairs[PrefetchIndex].AtomIndex);
			}

			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];

			// call the decoder directly (not through the vtable)
			AEFConstantKeyLerp<FORMAT>::GetBoneAtomTranslation(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

/**
 * Decompress all requested Scale components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
template<int32 FORMAT>
inline void AEFConstantKeyLerp<FORMAT>::GetPoseScales(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();

	if (PairCount == 0)
	{
		return;
	}

	if (bAnim_ConstantKeyLerp_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			const TArrayView<int32> ScaleOffsets = AnimData.CompressedScaleOffsets.OffsetData;
			const int32 StripSize = AnimData.CompressedScaleOffsets.StripSize;

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetConstantKeyLerpPoseScales(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				ScaleOffsets.GetData(),
				StripSize,
				AnimData.CompressedByteStream.GetData(),
				DecompContext.GetPlayableLength(),
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				FORMAT,
				PairCount);
		};
#endif
	}
	else
	{
		//@TODO: Verify that this prefetch is helping
		// Prefetch the desired pairs array and 2 destination spots; the loop will prefetch one 2 out each iteration
		FPlatformMisc::Prefetch(&(DesiredPairs[0]));
		const int32 PrefetchCount = FMath::Min(PairCount, 1);
		for (int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex)
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			FPlatformMisc::Prefetch(Atoms.GetData() + Pair.AtomIndex);
		}

		for (int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex)
		{
			int32 PrefetchIndex = PairIndex + PrefetchCount;
			if (PrefetchIndex < PairCount)
			{
				FPlatformMisc::Prefetch(Atoms.GetData() + DesiredPairs[PrefetchIndex].AtomIndex);
			}

			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];

			// call the decoder directly (not through the vtable)
			AEFConstantKeyLerp<FORMAT>::GetBoneAtomScale(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

template class AEFConstantKeyLerp<ACF_None>;
template class AEFConstantKeyLerp<ACF_Float96NoW>;
template class AEFConstantKeyLerp<ACF_Fixed48NoW>;
template class AEFConstantKeyLerp<ACF_IntervalFixed32NoW>;
template class AEFConstantKeyLerp<ACF_Fixed32NoW>;
template class AEFConstantKeyLerp<ACF_Float32NoW>;
template class AEFConstantKeyLerp<ACF_Identity>;

#endif
