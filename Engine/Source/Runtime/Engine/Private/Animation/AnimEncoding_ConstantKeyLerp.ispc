// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Vector.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"
#include "Animation/AnimationCompression.isph"

export void GetConstantKeyLerpPoseRotations(
	uniform FTransform Atoms[],
	const uniform BoneTrackPair DesiredPairs[],
	const uniform int CompressedTrackOffsets[],
	const uniform unsigned int8 CompressedByteStream[],
	const uniform float SequenceLength,
	const uniform float RelativePos,
	const uniform unsigned int8 InterpolationType,
	const uniform int FORMAT,
	const uniform int PairCount)
{
	const uniform int CompressedRotationStrideNum = CompressedRotationStrides[FORMAT] * CompressedRotationNum[FORMAT];
	const uniform int RotationStreamOffset = (FORMAT == ACF_IntervalFixed32NoW) ? (sizeof(uniform float)*6) : 0; // offset past Min and Range data

	foreach(PairIndex = 0 ... PairCount)
	{
		#pragma ignore warning(perf)
		const BoneTrackPair Pair = DesiredPairs[PairIndex];
		const int TrackIndex = Pair.TrackIndex;
		const int AtomIndex = Pair.AtomIndex;

		// call the decoder directly (not through the vtable)
		const int* TrackData = CompressedTrackOffsets + (TrackIndex * 4);

		#pragma ignore warning(perf)
		int RotKeysOffset = TrackData[2];

		#pragma ignore warning(perf)
		int NumRotKeys = TrackData[3];
		const unsigned int8* RotStream = CompressedByteStream + RotKeysOffset;

		FVector4f R0;

		if (NumRotKeys == 1)
		{
			// For a rotation track of n=1 keys, the single key is packed as an FQuatFloat96NoW.
			DecompressRotation(R0, RotStream, RotStream, ACF_Float96NoW);
		}
		else
		{
			int Index0;
			int Index1;
			const float Alpha = TimeToIndex(SequenceLength, RelativePos, NumRotKeys, InterpolationType, Index0, Index1);

			// unpack a single key first
			const unsigned int8* KeyData0= RotStream + RotationStreamOffset +(Index0*CompressedRotationStrideNum);
			DecompressRotation(R0, RotStream, KeyData0, FORMAT);

			if (Index0 != Index1)
			{
				// unpack and lerp between the two nearest keys
				const unsigned int8* KeyData1= RotStream + RotationStreamOffset +(Index1*CompressedRotationStrideNum);
				FVector4f R1;

				DecompressRotation(R1, RotStream, KeyData1, FORMAT);

				// Fast linear quaternion interpolation.
				const FVector4f BlendedQuat = QuatFastLerp(R0, R1, Alpha);
				R0 = VectorNormalizeQuaternion(BlendedQuat);
			}
		}

		uniform FVector4f AOSScratch[programCount];
		soa_to_aos4(R0.V[0], R0.V[1], R0.V[2], R0.V[3], (uniform float* uniform)&AOSScratch);
		foreach_active(i)
		{
			unmasked
			{
				uniform int uAtomIndex = extract(AtomIndex, i);
				uniform FVector4 *uniform BoneAtomRotation = (uniform FVector4 *uniform)&Atoms[uAtomIndex].Rotation;
				*BoneAtomRotation = ConvertVector4fTo4Native(AOSScratch[i]);
			}
		}
	}
}

export void GetConstantKeyLerpPoseTranslations(
	uniform FTransform Atoms[],
	const uniform BoneTrackPair DesiredPairs[],
	const uniform int CompressedTrackOffsets[],
	const uniform unsigned int8 CompressedByteStream[],
	const uniform float SequenceLength,
	const uniform float RelativePos,
	const uniform unsigned int8 InterpolationType,
	const uniform int FORMAT,
	const uniform int PairCount)
{
	const uniform int CompressedTranslationStrideNum = CompressedTranslationStrides[FORMAT] * CompressedTranslationNum[FORMAT];

	foreach(PairIndex = 0 ... PairCount)
	{
		#pragma ignore warning(perf)
		const BoneTrackPair Pair = DesiredPairs[PairIndex];
		const int TrackIndex = Pair.TrackIndex;
		const int AtomIndex = Pair.AtomIndex;

		const int* TrackData = CompressedTrackOffsets + (TrackIndex * 4);

		#pragma ignore warning(perf)
		const int TransKeysOffset = TrackData[0];

		#pragma ignore warning(perf)
		const int NumTransKeys = TrackData[1];
		const unsigned int8* TransStream = CompressedByteStream + TransKeysOffset;

		int32 Index0;
		int32 Index1;
		const float Alpha = TimeToIndex(SequenceLength, RelativePos, NumTransKeys, InterpolationType, Index0, Index1);

		const int32 TransStreamOffset = ((FORMAT == ACF_IntervalFixed32NoW) && NumTransKeys > 1) ? (sizeof(uniform float)*6) : 0; // offset past Min and Range data

		const unsigned int8* KeyData0 = TransStream + TransStreamOffset + Index0*CompressedTranslationStrideNum;
		FVector3f P0;
		DecompressTranslation(P0, TransStream, KeyData0, FORMAT);

		if (Index0 != Index1)
		{
			const unsigned int8* KeyData1 = TransStream + TransStreamOffset + Index1*CompressedTranslationStrideNum;
			FVector3f P1;

			DecompressTranslation(P1, TransStream, KeyData1, FORMAT);
			P0 = VectorLerp( P0, P1, Alpha );
		}

		uniform FVector4f AOSScratch[programCount];
		soa_to_aos4(P0.V[0], P0.V[1], P0.V[2], 0.0f, (uniform float* uniform)&AOSScratch);
		foreach_active(i)
		{
			unmasked
			{
				uniform int uAtomIndex = extract(AtomIndex, i);
				uniform FVector4 *uniform BoneAtomTranslation = (uniform FVector4 *uniform)&Atoms[uAtomIndex].Translation;
				*BoneAtomTranslation = ConvertVector4fTo4Native(AOSScratch[i]);
			}
		}
	}
}

export void GetConstantKeyLerpPoseScales(
	uniform FTransform Atoms[],
	const uniform BoneTrackPair DesiredPairs[],
	const uniform int CompressedScaleOffsets[],
	const uniform int StripSize,
	const uniform unsigned int8 CompressedByteStream[],
	const uniform float SequenceLength,
	const uniform float RelativePos,
	const uniform unsigned int8 InterpolationType,
	const uniform int FORMAT,
	const uniform int PairCount)
{
	const uniform int CompressedScaleStrideNum = CompressedScaleStrides[FORMAT] * CompressedScaleNum[FORMAT];

	foreach(PairIndex = 0 ... PairCount)
	{
		#pragma ignore warning(perf)
		const BoneTrackPair Pair = DesiredPairs[PairIndex];
		const int TrackIndex = Pair.TrackIndex;
		const int AtomIndex = Pair.AtomIndex;

		const int* ScaleData = CompressedScaleOffsets + (TrackIndex * StripSize);

		#pragma ignore warning(perf)
		const int ScaleKeysOffset = ScaleData[0];

		#pragma ignore warning(perf)
		const int NumScaleKeys = ScaleData[1];
		const unsigned int8* ScaleStream = CompressedByteStream + ScaleKeysOffset;

		int32 Index0;
		int32 Index1;
		const float Alpha = TimeToIndex(SequenceLength, RelativePos, NumScaleKeys, InterpolationType, Index0, Index1);

		const int32 ScaleStreamOffset = ((FORMAT == ACF_IntervalFixed32NoW) && NumScaleKeys > 1) ? (sizeof(uniform float)*6) : 0; // offset past Min and Range data

		const unsigned int8* KeyData0 = ScaleStream + ScaleStreamOffset + Index0*CompressedScaleStrideNum;
		FVector3f P0;
		DecompressScale(P0, ScaleStream, KeyData0, FORMAT);

		if (Index0 != Index1)
		{
			const unsigned int8* KeyData1 = ScaleStream + ScaleStreamOffset + Index1*CompressedScaleStrideNum;
			FVector3f P1;

			DecompressScale(P1, ScaleStream, KeyData1, FORMAT);
			P0 = VectorLerp( P0, P1, Alpha );
		}

		uniform FVector4f AOSScratch[programCount];
		soa_to_aos4(P0.V[0], P0.V[1], P0.V[2], 0.0f, (uniform float* uniform)&AOSScratch);
		foreach_active(i)
		{
			unmasked
			{
				uniform int uAtomIndex = extract(AtomIndex, i);
				uniform FVector4 *uniform BoneAtomScale = (uniform FVector4 *uniform)&Atoms[uAtomIndex].Scale3D;
				*BoneAtomScale = ConvertVector4fTo4Native(AOSScratch[i]);
			}
		}
	}
}
