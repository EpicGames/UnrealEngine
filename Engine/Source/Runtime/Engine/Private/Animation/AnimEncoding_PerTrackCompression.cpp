// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimEncoding_PerTrackCompression.cpp: Per-track decompressor
=============================================================================*/ 

#include "AnimEncoding_PerTrackCompression.h"
#include "Animation/AnimCompressionTypes.h"
#include "AnimationCompression.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#if INTEL_ISPC
#include "AnimEncoding_PerTrackCompression.ispc.generated.h"
#include "AutoRTFM.h"

static_assert(sizeof(ispc::FTransform) == sizeof(FTransform), "sizeof(ispc::FTransform) != sizeof(FTransform)");
static_assert(sizeof(ispc::BoneTrackPair) == sizeof(BoneTrackPair), "sizeof(ispc::BoneTrackPair) != sizeof(BoneTrackPair)");
#endif

#if !defined(ANIM_PER_TRACK_COMPRESSION_ISPC_ENABLED_DEFAULT)
#define ANIM_PER_TRACK_COMPRESSION_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bAnim_PerTrackCompression_ISPC_Enabled = INTEL_ISPC && ANIM_PER_TRACK_COMPRESSION_ISPC_ENABLED_DEFAULT;
#else
#include "HAL/IConsoleManager.h"
static bool bAnim_PerTrackCompression_ISPC_Enabled = ANIM_PER_TRACK_COMPRESSION_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarAnimPerTrackCompressionISPCEnabled(TEXT("a.PerTrackCompression.ISPC"), bAnim_PerTrackCompression_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in per track anim encoding"));
#endif

// This define controls whether scalar or vector code is used to decompress keys.  Note that not all key decompression code
// is vectorized yet, so some (seldom used) formats will actually get slower (due to extra LHS stalls) when enabled.
// The code also relies on a flexible permute instruction being available (e.g., PPC vperm)
#define USE_VECTOR_PTC_DECOMPRESSOR 0

#if USE_VECTOR_PTC_DECOMPRESSOR

// 32767, stored in 1X, plus biasing is 0
// Perm_Zeros takes the first 4 bytes of the second argument (which should be VectorZero() for this table use)
#define Perm_Zeros 0x10111213

#define Perm_X1 0x00010203
#define Perm_Y1 0x04050607
#define Perm_Z1 0x08090A0B
#define Perm_W2 0x1C1D1E1F

// One float96 key can be stored with implicit 0.0f components, this table (when indexed by format flags 0-3) plus one indicates how many bytes a key contains
static const uint8 Float96KeyBytesMinusOne[16] = { 11, 3, 3, 7, 3, 7, 7, 11, 11, 3, 3, 7, 3, 7, 7, 11 };

// One fixed48 key can be stored with implicit 0.0f components, this table (when indexed by format flags 0-3) plus one indicates how many bytes a key contains
static const uint8 Fixed48KeyBytesMinusOne[16] = { 5, 1, 1, 3, 1, 3, 3, 5, 5, 1, 1, 3, 1, 3, 3, 5 };

// One float96 translation key can be stored with implicit 0.0f components, this table (when indexed by format flags 0-2) indicates how to swizzle the 16 bytes read into one float each in X,Y,Z, and 0.0f in W
static const VectorRegister Trans96OptionalFormatPermMasks[8] =
{
	MakeVectorRegister( Perm_X1, Perm_Y1, Perm_Z1, (uint32)Perm_W2 ),  // 0 = 7 = all three valid
	MakeVectorRegister( Perm_X1, Perm_W2, Perm_W2, (uint32)Perm_W2 ),  // 1 = 001 = X
	MakeVectorRegister( Perm_W2, Perm_X1, Perm_W2, (uint32)Perm_W2 ),  // 2 = 010 = Y
	MakeVectorRegister( Perm_X1, Perm_Y1, Perm_W2, (uint32)Perm_W2 ),  // 3 = 011 = XY
	MakeVectorRegister( Perm_W2, Perm_W2, Perm_X1, (uint32)Perm_W2 ),  // 4 = 100 = Z
	MakeVectorRegister( Perm_X1, Perm_W2, Perm_Y1, (uint32)Perm_W2 ),  // 5 = 101 = XZ
	MakeVectorRegister( Perm_W2, Perm_X1, Perm_Y1, (uint32)Perm_W2 ),  // 6 = 110 = YZ
	MakeVectorRegister( Perm_X1, Perm_Y1, Perm_Z1, (uint32)Perm_W2 ),  // 7 = 111 = XYZ
};

#undef Perm_Zeros
#undef Perm_X1
#undef Perm_Y1
#undef Perm_Z1
#undef Perm_W2

// Perm_Zeros takes the first 4 bytes of the second argument (which should be DecompressPTCConstants, resulting in integer 32767, which becomes 0.0f after scaling and biasing)
#define Perm_Zeros 0x10111213

// Each of these takes two bytes of source data, and two zeros from the W of the second argument (which should be DecompressPTCConstants) to pad it out to a 4 byte int32
#define Perm_Data1 0x1F1F0001
#define Perm_Data2 0x1F1F0203
#define Perm_Data3 0x1F1F0405

// One fixed48 rotation key can be stored with implicit 0.0f components, this table (when indexed by format flags 0-2) indicates how to swizzle the 16 bytes read into one short each in X,Y,Z)
static const VectorRegister Fix48FormatPermMasks[8] =
{
	MakeVectorRegister( Perm_Data1, Perm_Data2, Perm_Data3, (uint32)Perm_Zeros ),  // 0 = 7 = all three valid
	MakeVectorRegister( Perm_Data1, Perm_Zeros, Perm_Zeros, (uint32)Perm_Zeros ),  // 1 = 001 = X
	MakeVectorRegister( Perm_Zeros, Perm_Data1, Perm_Zeros, (uint32)Perm_Zeros ),  // 2 = 010 = Y
	MakeVectorRegister( Perm_Data1, Perm_Data2, Perm_Zeros, (uint32)Perm_Zeros ),  // 3 = 011 = XY
	MakeVectorRegister( Perm_Zeros, Perm_Zeros, Perm_Data1, (uint32)Perm_Zeros ),  // 4 = 100 = Z
	MakeVectorRegister( Perm_Data1, Perm_Zeros, Perm_Data2, (uint32)Perm_Zeros ),  // 5 = 101 = XZ
	MakeVectorRegister( Perm_Zeros, Perm_Data1, Perm_Data2, (uint32)Perm_Zeros ),  // 6 = 110 = YZ
	MakeVectorRegister( Perm_Data1, Perm_Data2, Perm_Data3, (uint32)Perm_Zeros ),  // 7 = 111 = XYZ
};

#undef Perm_Zeros
#undef Perm_Data1
#undef Perm_Data2
#undef Perm_Data3

// Constants used when decompressing fixed48 translation keys (pre-scale, pre-bias integer representation of a final output 0.0f)
static const VectorRegister DecompressPTCTransConstants = MakeVectorRegister( 255, 255, 255, (uint32)0 );

// Constants used when decompressing fixed48 rotation keys (pre-scale, pre-bias integer representation of a final output 0.0f)
static const VectorRegister DecompressPTCConstants = MakeVectorRegister( 32767, 32767, 32767, (uint32)0 );

// Scale-bias factors for decompressing fixed48 data (both rotation and translation)
static const VectorRegister BiasFix48Data = MakeVectorRegister( -32767.0f, -32767.0f, -32767.0f, -32767.0f );
static const VectorRegister ScaleRotData = MakeVectorRegister( 3.0518509475997192297128208258309e-5f, 3.0518509475997192297128208258309e-5f, 3.0518509475997192297128208258309e-5f, 1.0f );

//@TODO: Looks like fixed48 for translation is basically broken right now (using 8 bits instead of 16 bits!).  The scale is omitted below because it's all 1's
static const VectorRegister BiasTransData =  MakeVectorRegister( -255.0f, -255.0f, -255.0f, -255.0f );
static const VectorRegister ScaleTransData = MakeVectorRegister( 1.0f, 1.0f, 1.0f, 1.0f );

/** Decompress a single translation key from a single track that was compressed with the PerTrack codec (vectorized) */
static FORCEINLINE_DEBUGGABLE VectorRegister DecompressSingleTrackTranslationVectorized(int32 Format, int32 FormatFlags, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
{
	if( Format == ACF_Float96NoW )
	{
		const VectorRegister KeyJumbled = VectorLoadNPlusOneUnalignedBytes(KeyData, Float96KeyBytesMinusOne[FormatFlags]);
		const VectorRegister XYZ = VectorPermute(KeyJumbled, VectorZero(), Trans96OptionalFormatPermMasks[FormatFlags & 7]);

		return XYZ;
	}
	else if (Format == ACF_Fixed48NoW)
	{
		const VectorRegister KeyJumbled = VectorLoadNPlusOneUnalignedBytes(KeyData, Fixed48KeyBytesMinusOne[FormatFlags]);
		const VectorRegister Key = VectorPermute(KeyJumbled, DecompressPTCTransConstants, Fix48FormatPermMasks[FormatFlags & 7]);
		const VectorRegister FPKey = VectorUitof(Key);

		const VectorRegister BiasedData = VectorAdd(FPKey, BiasTransData);
		//const VectorRegister XYZ = VectorMultiply(BiasedData, ScaleTransData);
		const VectorRegister XYZ = BiasedData;

		return XYZ;
	}
	else if (Format == ACF_IntervalFixed32NoW)
	{
		const float* RESTRICT SourceBounds = (float*)TopOfStream;

		float Mins[3] = {0.0f, 0.0f, 0.0f};
		float Ranges[3] = {0.0f, 0.0f, 0.0f};

		if (FormatFlags & 1)
		{
			Mins[0] = *SourceBounds++;
			Ranges[0] = *SourceBounds++;
		}
		if (FormatFlags & 2)
		{
			Mins[1] = *SourceBounds++;
			Ranges[1] = *SourceBounds++;
		}
		if (FormatFlags & 4)
		{
			Mins[2] = *SourceBounds++;
			Ranges[2] = *SourceBounds++;
		}

		// This one is still used for ~4% of the cases, so making it faster would be nice
		FVector Out;
		((FVectorIntervalFixed32NoW*)KeyData)->ToVector(Out, Mins, Ranges);
		return VectorLoadAligned(&Out);
	}

	return VectorZero();
}

/** Decompress a single Scale key from a single track that was compressed with the PerTrack codec (vectorized) */
static FORCEINLINE_DEBUGGABLE VectorRegister DecompressSingleTrackScaleVectorized(int32 Format, int32 FormatFlags, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
{
	if( Format == ACF_Float96NoW )
	{
		const VectorRegister KeyJumbled = VectorLoadNPlusOneUnalignedBytes(KeyData, Float96KeyBytesMinusOne[FormatFlags]);
		const VectorRegister XYZ = VectorPermute(KeyJumbled, VectorZero(), Trans96OptionalFormatPermMasks[FormatFlags & 7]);

		return XYZ;
	}
	else if (Format == ACF_Fixed48NoW)
	{
		const VectorRegister KeyJumbled = VectorLoadNPlusOneUnalignedBytes(KeyData, Fixed48KeyBytesMinusOne[FormatFlags]);
		const VectorRegister Key = VectorPermute(KeyJumbled, DecompressPTCTransConstants, Fix48FormatPermMasks[FormatFlags & 7]);
		const VectorRegister FPKey = VectorUitof(Key);

		const VectorRegister BiasedData = VectorAdd(FPKey, BiasTransData);
		//const VectorRegister XYZ = VectorMultiply(BiasedData, ScaleTransData);
		const VectorRegister XYZ = BiasedData;

		return XYZ;
	}
	else if (Format == ACF_IntervalFixed32NoW)
	{
		const float* RESTRICT SourceBounds = (float*)TopOfStream;

		float Mins[3] = {0.0f, 0.0f, 0.0f};
		float Ranges[3] = {0.0f, 0.0f, 0.0f};

		if (FormatFlags & 1)
		{
			Mins[0] = *SourceBounds++;
			Ranges[0] = *SourceBounds++;
		}
		if (FormatFlags & 2)
		{
			Mins[1] = *SourceBounds++;
			Ranges[1] = *SourceBounds++;
		}
		if (FormatFlags & 4)
		{
			Mins[2] = *SourceBounds++;
			Ranges[2] = *SourceBounds++;
		}

		// This one is still used for ~4% of the cases, so making it faster would be nice
		FVector Out;
		((FVectorIntervalFixed32NoW*)KeyData)->ToVector(Out, Mins, Ranges);
		return VectorLoadAligned(&Out);
	}

	return VectorZero();
}

/** Decompress a single rotation key from a single track that was compressed with the PerTrack codec (vectorized) */
static FORCEINLINE_DEBUGGABLE VectorRegister DecompressSingleTrackRotationVectorized(int32 Format, int32 FormatFlags, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
{
	if (Format == ACF_Fixed48NoW)
	{
		const VectorRegister KeyJumbled = VectorLoadNPlusOneUnalignedBytes(KeyData, Fixed48KeyBytesMinusOne[FormatFlags]);
		const VectorRegister Key = VectorPermute(KeyJumbled, DecompressPTCConstants, Fix48FormatPermMasks[FormatFlags & 7]);
		const VectorRegister FPKey = VectorUitof(Key);

		const VectorRegister BiasedData = VectorAdd(FPKey, BiasFix48Data);
		const VectorRegister XYZ = VectorMultiply(BiasedData, ScaleRotData);
		const VectorRegister LengthSquared = VectorDot3(XYZ, XYZ);

		const VectorRegister WSquared = VectorSubtract(VectorOne(), LengthSquared);
		const VectorRegister WSquaredSqrt = VectorReciprocal(VectorReciprocalSqrt(WSquared));
		const VectorRegister WWWW = VectorSelect(VectorCompareGT(WSquared, VectorZero()), WSquaredSqrt, VectorZero());

		const VectorRegister XYZW = VectorMergeVecXYZ_VecW(XYZ, WWWW);

		return XYZW;
	}
	else if (Format == ACF_Float96NoW)
	{
		const VectorRegister XYZ = VectorLoadFloat3(KeyData);
		const VectorRegister LengthSquared = VectorDot3(XYZ, XYZ);

		const VectorRegister WSquared = VectorSubtract(VectorOne(), LengthSquared);
		const VectorRegister WSquaredSqrt = VectorReciprocal(VectorReciprocalSqrt(WSquared));
		const VectorRegister WWWW = VectorSelect(VectorCompareGT(WSquared, VectorZero()), WSquaredSqrt, VectorZero());

		const VectorRegister XYZW = VectorMergeVecXYZ_VecW(XYZ, WWWW);

		return XYZW;
	}
	else if ( Format == ACF_IntervalFixed32NoW )
	{
		const float* RESTRICT SourceBounds = (float*)TopOfStream;

		float Mins[3] = {0.0f, 0.0f, 0.0f};
		float Ranges[3] = {0.0f, 0.0f, 0.0f};

		if (FormatFlags & 1)
		{
			Mins[0] = *SourceBounds++;
			Ranges[0] = *SourceBounds++;
		}
		if (FormatFlags & 2)
		{
			Mins[1] = *SourceBounds++;
			Ranges[1] = *SourceBounds++;
		}
		if (FormatFlags & 4)
		{
			Mins[2] = *SourceBounds++;
			Ranges[2] = *SourceBounds++;
		}

		// This one is still used for ~4% of the cases, so making it faster would be nice
		FQuat Out;
		((FQuatIntervalFixed32NoW*)KeyData)->ToQuat( Out, Mins, Ranges );
		return VectorLoadAligned(&Out);
	}
	else if ( Format == ACF_Float32NoW )
	{
		// This isn't used for compression anymore so making it fast isn't very important
		FQuat Out;
		((FQuatFloat32NoW*)KeyData)->ToQuat( Out );
		return VectorLoadAligned(&Out);
	}
	else if (Format == ACF_Fixed32NoW)
	{
		// This isn't used for compression anymore so making it fast isn't very important
		FQuat Out;
		((FQuatFixed32NoW*)KeyData)->ToQuat(Out);
		return VectorLoadAligned(&Out);
	}

	return VectorLoadAligned(&(FQuat::Identity));
}

#endif // USE_VECTOR_PTC_DECOMPRESSOR

template<class TArchive>
void AEFPerTrackCompressionCodec::ByteSwapOneTrack(FUECompressedAnimData& CompressedData, TArchive& MemoryStream, int32 BufferStart, int32 Offset)
{
	// Translation data.
	if (Offset != INDEX_NONE)
	{
		checkSlow( (Offset % 4) == 0 && "CompressedByteStream not aligned to four bytes" );

		MemoryStream.Seek(BufferStart+Offset);

		uint8* TrackData = CompressedData.CompressedByteStream.GetData() + Offset;

		// Read the header
		AC_UnalignedSwap(MemoryStream, TrackData, sizeof(int32));

		const int32 Header = *(reinterpret_cast<int32*>(TrackData - sizeof(int32)));


		int32 KeyFormat;
		int32 NumKeys;
		int32 FormatFlags;
		FAnimationCompression_PerTrackUtils::DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags);

		int32 FixedComponentSize = 0;
		int32 FixedComponentCount = 0;
		int32 KeyComponentSize = 0;
		int32 KeyComponentCount = 0;
		FAnimationCompression_PerTrackUtils::GetAllSizesFromFormat(KeyFormat, FormatFlags, /*OUT*/ KeyComponentCount, /*OUT*/ KeyComponentSize, /*OUT*/ FixedComponentCount, /*OUT*/ FixedComponentSize);

		check(FixedComponentSize != 0);
		check(KeyComponentSize != 0);

		// Handle per-track metadata
		for (int32 i = 0; i < FixedComponentCount; ++i)
		{
			AC_UnalignedSwap(MemoryStream, TrackData, FixedComponentSize);
		}

		// Handle keys
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			for (int32 i = 0; i < KeyComponentCount; ++i)
			{
				AC_UnalignedSwap(MemoryStream, TrackData, KeyComponentSize);
			}
		}

		// Handle the key frame table if present
		if ((FormatFlags & 0x8) != 0)
		{
			// Make sure the key->frame table is 4 byte aligned
			PreservePadding(TrackData, MemoryStream);

			const int32 FrameTableEntrySize = (CompressedData.CompressedNumberOfKeys <= 0xFF) ? sizeof(uint8) : sizeof(uint16);
			for (int32 i = 0; i < NumKeys; ++i)
			{
				AC_UnalignedSwap(MemoryStream, TrackData, FrameTableEntrySize);
			}
		}

		// Make sure the next track is 4 byte aligned
		PreservePadding(TrackData, MemoryStream);
	}
}

template void AEFPerTrackCompressionCodec::ByteSwapOneTrack(FUECompressedAnimData& CompressedData, FMemoryReader& MemoryStream, int32 BufferStart, int32 Offset);
template void AEFPerTrackCompressionCodec::ByteSwapOneTrack(FUECompressedAnimData& CompressedData, FMemoryWriter& MemoryStream, int32 BufferStart, int32 Offset);


/**
 * Preserves 4 byte alignment within a stream
 *
 * @param	TrackData [inout]	The current data offset (will be returned four byte aligned from the start of the compressed byte stream)
 * @param	MemoryStream		The MemoryReader or MemoryWriter object to read from/write to.
 */
void AEFPerTrackCompressionCodec::PreservePadding(uint8*& TrackData, FMemoryArchive& MemoryStream)
{
	// Preserve padding
	const PTRINT ByteStreamLoc = (PTRINT) TrackData;
	const int32 PadCount = static_cast<int32>( Align(ByteStreamLoc, 4) - ByteStreamLoc );
	if (MemoryStream.IsSaving())
	{
		const uint8 PadSentinel = 85; // (1<<1)+(1<<3)+(1<<5)+(1<<7)

		for (int32 PadByteIndex = 0; PadByteIndex < PadCount; ++PadByteIndex)
		{
			MemoryStream.Serialize( (void*)&PadSentinel, sizeof(uint8) );
		}
		TrackData += PadCount;
	}
	else
	{
		MemoryStream.Serialize(TrackData, PadCount);
		TrackData += PadCount;
	}
}

/**
 * Handles Byte-swapping incoming animation data from a MemoryReader
 *
 * @param	CompressedData		The compressed animation data being operated on.
 * @param	MemoryReader		The MemoryReader object to read from.
 */
void AEFPerTrackCompressionCodec::ByteSwapIn(
	FUECompressedAnimData& CompressedData,
	FMemoryReader& MemoryReader)
{
	const int64 BufferStart = MemoryReader.Tell();
	int64 OriginalNumBytes = MemoryReader.TotalSize() - BufferStart;
	check(OriginalNumBytes < INT_MAX);
	int32 OriginalNumBytes32 = (int32)OriginalNumBytes;
	check(CompressedData.CompressedByteStream.Num() == OriginalNumBytes);

	const int32 NumTracks = CompressedData.CompressedTrackOffsets.Num() / 2;
	const bool bHasScaleData = CompressedData.CompressedScaleOffsets.IsValid();

	for ( int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex )
	{
		const int32 OffsetTrans = CompressedData.CompressedTrackOffsets[TrackIndex*2+0];
		ByteSwapOneTrack(CompressedData, MemoryReader, BufferStart, OffsetTrans);

		const int32 OffsetRot = CompressedData.CompressedTrackOffsets[TrackIndex*2+1];
		ByteSwapOneTrack(CompressedData, MemoryReader, BufferStart, OffsetRot);
		if (bHasScaleData)
		{
			const int32 OffsetScale = CompressedData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 0);
			ByteSwapOneTrack(CompressedData, MemoryReader, BufferStart, OffsetScale);
		}
	}
}


/**
 * Handles Byte-swapping outgoing animation data to an array of BYTEs
 *
 * @param	CompressedData		The compressed animation data being operated on.
 * @param	SerializedData		The output buffer.
 * @param	ForceByteSwapping	true is byte swapping is not optional.
 */
void AEFPerTrackCompressionCodec::ByteSwapOut(
	FUECompressedAnimData& CompressedData,
	FMemoryWriter& MemoryWriter)
{
	const int32 BufferStart = MemoryWriter.Tell();

	const int32 NumTracks = CompressedData.CompressedTrackOffsets.Num() / 2;
	const bool bHasScaleData = CompressedData.CompressedScaleOffsets.IsValid();
	for ( int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex )
	{
		const int32 OffsetTrans = CompressedData.CompressedTrackOffsets[TrackIndex*2+0];
		ByteSwapOneTrack(CompressedData, MemoryWriter, BufferStart, OffsetTrans);

		const int32 OffsetRot = CompressedData.CompressedTrackOffsets[TrackIndex*2+1];
		ByteSwapOneTrack(CompressedData, MemoryWriter, BufferStart, OffsetRot);

		if (bHasScaleData)
		{
			const int32 OffsetScale = CompressedData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 0);
			ByteSwapOneTrack(CompressedData, MemoryWriter, BufferStart, OffsetScale);
		}
	}
}


void AEFPerTrackCompressionCodec::GetBoneAtomRotation(
	FTransform& OutAtom,
	FAnimSequenceDecompressionContext& DecompContext,
	int32 TrackIndex)
{
	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);
	const int32* RESTRICT TrackOffsetData = AnimData.CompressedTrackOffsets.GetData() + (TrackIndex * 2);
	const int32 RotKeysOffset = TrackOffsetData[1];

	if (RotKeysOffset != INDEX_NONE)
	{
		const uint8* RESTRICT TrackData = AnimData.CompressedByteStream.GetData() + RotKeysOffset + 4;
		const int32 Header = *((int32*)(AnimData.CompressedByteStream.GetData() + RotKeysOffset));

		int32 KeyFormat;
		int32 NumKeys;
		int32 FormatFlags;
		int32 BytesPerKey;
		int32 FixedBytes;
		FAnimationCompression_PerTrackUtils::DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/BytesPerKey, /*OUT*/ FixedBytes);

		// Figure out the key indexes
		int32 Index0 = 0;
		int32 Index1 = 0;

		// Alpha is volatile to force the compiler to store it to memory immediately, so it is ready to be loaded into a vector register without a LHS after decompressing a track 
		volatile float Alpha = 0.0f;

		if (NumKeys > 1)
		{
			if ((FormatFlags & 0x8) == 0)
			{
				Alpha = TimeToIndex(DecompContext.GetPlayableLength(), DecompContext.GetRelativePosition(), NumKeys, DecompContext.Interpolation, Index0, Index1);
			}
			else
			{
				const uint8* RESTRICT FrameTable = Align(TrackData + FixedBytes + BytesPerKey * NumKeys, 4);
				Alpha = TimeToIndex(DecompContext.Interpolation, AnimData.CompressedNumberOfKeys, FrameTable, DecompContext.GetRelativePosition(), NumKeys, Index0, Index1);
			}
		}

		// Unpack the first key
		const uint8* RESTRICT KeyData0 = TrackData + FixedBytes + (Index0 * BytesPerKey);

#if USE_VECTOR_PTC_DECOMPRESSOR
		const VectorRegister R0 = DecompressSingleTrackRotationVectorized(KeyFormat, FormatFlags, TrackData, KeyData0);
#else
		FQuat4f R0;
		FAnimationCompression_PerTrackUtils::DecompressRotation(KeyFormat, FormatFlags, R0, TrackData, KeyData0);
#endif

		// If there is a second key, figure out the lerp between the two of them
		if (Index0 != Index1)
		{
			const uint8* RESTRICT KeyData1 = TrackData + FixedBytes + (Index1 * BytesPerKey);

#if USE_VECTOR_PTC_DECOMPRESSOR
			ScalarRegister VAlpha(static_cast<float>(Alpha));

			const VectorRegister R1 = DecompressSingleTrackRotationVectorized(KeyFormat, FormatFlags, TrackData, KeyData1);

			const VectorRegister BlendedQuat = VectorLerpQuat(R0, R1, VAlpha);
			const VectorRegister BlendedNormalizedQuat = VectorNormalizeQuaternion(BlendedQuat);
			OutAtom.SetRotation(BlendedNormalizedQuat);
#else
			FQuat4f R1;
			FAnimationCompression_PerTrackUtils::DecompressRotation(KeyFormat, FormatFlags, R1, TrackData, KeyData1);

			// Fast linear quaternion interpolation.
			FQuat4f BlendedQuat = FQuat4f::FastLerp(R0, R1, Alpha);
			OutAtom.SetRotation( FQuat(BlendedQuat) );
			OutAtom.NormalizeRotation();
#endif
		}
		else // (Index0 == Index1)
		{
			OutAtom.SetRotation( FQuat(R0) );
			OutAtom.NormalizeRotation();
		}
	}
	else
	{
		// Identity track
		OutAtom.SetRotation(FQuat::Identity);
	}
}


void AEFPerTrackCompressionCodec::GetBoneAtomTranslation(
	FTransform& OutAtom,
	FAnimSequenceDecompressionContext& DecompContext,
	int32 TrackIndex)
{
	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);
	const int32* RESTRICT TrackOffsetData = AnimData.CompressedTrackOffsets.GetData() + (TrackIndex * 2);
	const int32 PosKeysOffset = TrackOffsetData[0];

	if (PosKeysOffset != INDEX_NONE)
	{
		const uint8* RESTRICT TrackData = AnimData.CompressedByteStream.GetData() + PosKeysOffset + 4;
		const int32 Header = *((int32*)(AnimData.CompressedByteStream.GetData() + PosKeysOffset));

		int32 KeyFormat;
		int32 NumKeys;
		int32 FormatFlags;
		int32 BytesPerKey;
		int32 FixedBytes;
		FAnimationCompression_PerTrackUtils::DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/BytesPerKey, /*OUT*/ FixedBytes);

		checkf(KeyFormat != ACF_None, TEXT("[%s] contians invalid keyformat. NumKeys (%d), FormatFlags (%d), BytesPerKeys (%d), FixedBytes (%d)"), *DecompContext.AnimName.ToString(), NumKeys, FormatFlags, BytesPerKey, FixedBytes);

		checkf(KeyFormat < ACF_MAX, TEXT("[%s] contians invalid keyformat. NumKeys (%d), FormatFlags (%d), BytesPerKeys (%d), FixedBytes (%d), PosKeysOffset (%d), TrackIndex (%d)"), *DecompContext.AnimName.ToString(), NumKeys, FormatFlags, BytesPerKey, FixedBytes, PosKeysOffset, TrackIndex)
		// Figure out the key indexes
		int32 Index0 = 0;
		int32 Index1 = 0;

		// Alpha is volatile to force the compiler to store it to memory immediately, so it is ready to be loaded into a vector register without a LHS after decompressing a track 
		volatile float Alpha = 0.0f;

		if (NumKeys > 1)
		{
			if ((FormatFlags & 0x8) == 0)
			{
				Alpha = TimeToIndex(DecompContext.GetPlayableLength(), DecompContext.GetRelativePosition(), NumKeys, DecompContext.Interpolation, Index0, Index1);
			}
			else
			{
				const uint8* RESTRICT FrameTable = Align(TrackData + FixedBytes + BytesPerKey * NumKeys, 4);
				Alpha = TimeToIndex(DecompContext.Interpolation, AnimData.CompressedNumberOfKeys, FrameTable, DecompContext.GetRelativePosition(), NumKeys, Index0, Index1);
			}
		}

		// Unpack the first key
		const uint8* RESTRICT KeyData0 = TrackData + FixedBytes + (Index0 * BytesPerKey);

#if USE_VECTOR_PTC_DECOMPRESSOR
		const VectorRegister R0 = DecompressSingleTrackTranslationVectorized(KeyFormat, FormatFlags, TrackData, KeyData0);
#else
		FVector3f R0;
		FAnimationCompression_PerTrackUtils::DecompressTranslation(KeyFormat, FormatFlags, R0, TrackData, KeyData0);
#endif

		// If there is a second key, figure out the lerp between the two of them
		if (Index0 != Index1)
		{
			const uint8* RESTRICT KeyData1 = TrackData + FixedBytes + (Index1 * BytesPerKey);

#if USE_VECTOR_PTC_DECOMPRESSOR
			ScalarRegister VAlpha(static_cast<float>(Alpha));

			const VectorRegister R1 = DecompressSingleTrackTranslationVectorized(KeyFormat, FormatFlags, TrackData, KeyData1);

			const VectorRegister BlendedTranslation = FMath::Lerp(R0, R1, VAlpha);
			OutAtom.SetTranslation(BlendedTranslation);
#else
			FVector3f R1;
			FAnimationCompression_PerTrackUtils::DecompressTranslation(KeyFormat, FormatFlags, R1, TrackData, KeyData1);

			OutAtom.SetTranslation((FVector)FMath::Lerp(R0, R1, Alpha));
#endif
		}
		else // (Index0 == Index1)
		{
			OutAtom.SetTranslation((FVector)R0);
		}
	}
	else
	{
		// Identity track
#if USE_VECTOR_PTC_DECOMPRESSOR
		OutAtom.SetTranslation(VectorZero());
#else
		OutAtom.SetTranslation(FVector::ZeroVector);
#endif
	}
}


void AEFPerTrackCompressionCodec::GetBoneAtomScale(
	FTransform& OutAtom,
	FAnimSequenceDecompressionContext& DecompContext,
	int32 TrackIndex)
{
	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);
	const int32 ScaleKeysOffset = AnimData.CompressedScaleOffsets.GetOffsetData( TrackIndex, 0 );

	if (ScaleKeysOffset != INDEX_NONE)
	{
		const uint8* RESTRICT TrackData = AnimData.CompressedByteStream.GetData() + ScaleKeysOffset + 4;
		const int32 Header = *((int32*)(AnimData.CompressedByteStream.GetData() + ScaleKeysOffset));

		int32 KeyFormat;
		int32 NumKeys;
		int32 FormatFlags;
		int32 BytesPerKey;
		int32 FixedBytes;
		FAnimationCompression_PerTrackUtils::DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/BytesPerKey, /*OUT*/ FixedBytes);

		// Figure out the key indexes
		int32 Index0 = 0;
		int32 Index1 = 0;

		// Alpha is volatile to force the compiler to store it to memory immediately, so it is ready to be loaded into a vector register without a LHS after decompressing a track 
		volatile float Alpha = 0.0f;

		if (NumKeys > 1)
		{
			if ((FormatFlags & 0x8) == 0)
			{
				Alpha = TimeToIndex(DecompContext.GetPlayableLength(), DecompContext.GetRelativePosition(), NumKeys, DecompContext.Interpolation, Index0, Index1);
			}
			else
			{
				const uint8* RESTRICT FrameTable = Align(TrackData + FixedBytes + BytesPerKey * NumKeys, 4);
				Alpha = TimeToIndex(DecompContext.Interpolation, AnimData.CompressedNumberOfKeys, FrameTable, DecompContext.GetRelativePosition(), NumKeys, Index0, Index1);
			}
		}

		// Unpack the first key
		const uint8* RESTRICT KeyData0 = TrackData + FixedBytes + (Index0 * BytesPerKey);

#if USE_VECTOR_PTC_DECOMPRESSOR
		const VectorRegister R0 = DecompressSingleTrackScaleVectorized(KeyFormat, FormatFlags, TrackData, KeyData0);
#else
		FVector3f R0;
		FAnimationCompression_PerTrackUtils::DecompressScale(KeyFormat, FormatFlags, R0, TrackData, KeyData0);
#endif

		// If there is a second key, figure out the lerp between the two of them
		if (Index0 != Index1)
		{
			const uint8* RESTRICT KeyData1 = TrackData + FixedBytes + (Index1 * BytesPerKey);

#if USE_VECTOR_PTC_DECOMPRESSOR
			ScalarRegister VAlpha(static_cast<float>(Alpha));

			const VectorRegister R1 = DecompressSingleTrackScaleVectorized(KeyFormat, FormatFlags, TrackData, KeyData1);

			const VectorRegister BlendedScale = FMath::Lerp(R0, R1, VAlpha);
			OutAtom.SetScale(BlendedScale);
#else
			FVector3f R1;
			FAnimationCompression_PerTrackUtils::DecompressScale(KeyFormat, FormatFlags, R1, TrackData, KeyData1);

			OutAtom.SetScale3D((FVector)FMath::Lerp(R0, R1, Alpha));
#endif
		}
		else // (Index0 == Index1)
		{
			OutAtom.SetScale3D((FVector)R0);
		}
	}
	else
	{
		// Identity track
#if USE_VECTOR_PTC_DECOMPRESSOR
		OutAtom.SetScale(VectorZero());
#else
		OutAtom.SetScale3D(FVector::ZeroVector);
#endif
	}
}

#if USE_ANIMATION_CODEC_BATCH_SOLVER

/**
 * Decompress all requested rotation components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
void AEFPerTrackCompressionCodec::GetPoseRotations(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();

	if (PairCount == 0)
	{
		return;
	}

	if (bAnim_PerTrackCompression_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetPerTrackCompressionPoseRotations(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				AnimData.CompressedTrackOffsets.GetData(),
				AnimData.CompressedByteStream.GetData(),
				AnimData.CompressedNumberOfKeys,
				DecompContext.GetPlayableLength(),
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				PairCount);
		};
#endif
	}
	else
	{
		for( int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex )
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];
	
			GetBoneAtomRotation(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

/**
 * Decompress all requested translation components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
void AEFPerTrackCompressionCodec::GetPoseTranslations(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();
	if (PairCount == 0)
	{
		return;
	}
	if (bAnim_PerTrackCompression_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetPerTrackCompressionPoseTranslations(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				AnimData.CompressedTrackOffsets.GetData(),
				AnimData.CompressedByteStream.GetData(),
				AnimData.CompressedNumberOfKeys,
				DecompContext.GetPlayableLength(),
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				PairCount);
		};
#endif
	}
	else
	{
		for( int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex )
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];
	
			GetBoneAtomTranslation(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

/**
 * Decompress all requested Scale components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
void AEFPerTrackCompressionCodec::GetPoseScales(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();

	if (PairCount == 0)
	{
		return;
	}

	if (bAnim_PerTrackCompression_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			const TArrayView<int32> ScaleOffsets = AnimData.CompressedScaleOffsets.OffsetData;
			const int32 StripSize = AnimData.CompressedScaleOffsets.StripSize;

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetPerTrackCompressionPoseScales(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				ScaleOffsets.GetData(),
				StripSize,
				AnimData.CompressedByteStream.GetData(),
				AnimData.CompressedNumberOfKeys,
				DecompContext.GetPlayableLength(),
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				PairCount);
		};
#endif
	}
	else
	{
		for( int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex )
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];
	
			GetBoneAtomScale(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

#endif // USE_ANIMATION_CODEC_BATCH_SOLVER
