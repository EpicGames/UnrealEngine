// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Vector.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"
#include "Animation/AnimationCompression.isph"

#define INDEX_NONE -1

/**
 * Figures out the size of various parts of a compressed track from the format and format flags combo
 *   @param KeyFormat		The encoding format used for each key
 *   @param FormatFlags		Three bits of format-specific information and a single bit to indicate if a key->frame table follows the keys
 *
 */
static inline void GetAllSizesFromFormat(const int KeyFormat, const int FormatFlags,
	int& KeyComponentCount, int& KeyComponentSize,
	int& FixedComponentCount, int& FixedComponentSize)
{
	// Note: this method can be used for translation too, but animation sequences compressed with this codec will
	// use ACF_Float96NoW for uncompressed translation, so using the rotation table is still valid
	#pragma ignore warning(perf)
	KeyComponentSize = CompressedRotationStrides[KeyFormat];
	FixedComponentSize = sizeof(uniform float);

	const unsigned int Index = (unsigned int)((FormatFlags & 0x7) | (KeyFormat << 3));

	#pragma ignore warning(perf)
	const unsigned int8 PerTrackComponentLookup = PerTrackNumComponentTable[Index];
	const int32 ComponentLookup = (int)PerTrackComponentLookup;

	if (KeyFormat != ACF_IntervalFixed32NoW)
	{
		FixedComponentCount = 0;
		KeyComponentCount = ComponentLookup;
	}
	else
	{
		// Min/Range floats for all non-zero channels
		FixedComponentCount = ComponentLookup;
		KeyComponentCount = 1;
	}
}

static inline void GetByteSizesFromFormat(const int KeyFormat, const int FormatFlags, int& BytesPerKey, int& FixedBytes)
{
	int FixedComponentSize = 0;
	int FixedComponentCount = 0;
	int KeyComponentSize = 0;
	int KeyComponentCount = 0;

	GetAllSizesFromFormat(KeyFormat, FormatFlags, /*OUT*/ KeyComponentCount, /*OUT*/ KeyComponentSize, /*OUT*/ FixedComponentCount, /*OUT*/ FixedComponentSize);

	BytesPerKey = KeyComponentCount * KeyComponentSize;
	FixedBytes = FixedComponentCount * FixedComponentSize;
}

/**
 * Decomposes a header created with MakeHeader into three/four fields (two are still left packed into FormatFlags):
 *   @param Header				The header to decompose
 *   @param KeyFormat [OUT]		The encoding format used for each key
 *   @param	NumKeys	[OUT]		The number of keys in this track
 *   @param FormatFlags [OUT]	Three bits of format-specific information and a single bit to indicate if a key->frame table follows the keys
 */
static inline void DecomposeHeader(const int Header, int& KeyFormat, int& NumKeys, int& FormatFlags)
{
	NumKeys = Header & 0x00FFFFFF;
	FormatFlags = (Header >> 24) & 0x0F;
	KeyFormat = (Header >> 28) & 0x0F;
}

/**
 * Decomposes a header created with MakeHeader into three/four fields (two are still left packed into FormatFlags):
 *   @param Header				The header to decompose
 *   @param KeyFormat [OUT]		The encoding format used for each key
 *   @param	NumKeys	[OUT]		The number of keys in this track
 *   @param FormatFlags [OUT]	Three bits of format-specific information and a single bit to indicate if a key->frame table follows the keys
 *
 * And some derived values:
 *   @param	BytesPerKey [OUT]	The number of bytes each key takes up
 *   @param	FixedBytes [OUT]	The number of fixed bytes at the head of the track stream
 */
static inline void DecomposeHeader(const int Header, int& KeyFormat, int& NumKeys, int& FormatFlags, int& BytesPerKey, int& FixedBytes)
{
	NumKeys = Header & 0x00FFFFFF;
	FormatFlags = (Header >> 24) & 0x0F;
	KeyFormat = (Header >> 28) & 0x0F;

	// Figure out the component sizes / counts (they can be changed per-format)
	GetByteSizesFromFormat(KeyFormat, FormatFlags, /*OUT*/ BytesPerKey, /*OUT*/ FixedBytes);
}

export void GetPerTrackCompressionPoseRotations(
	uniform FTransform Atoms[],
	const uniform BoneTrackPair DesiredPairs[],
	const uniform int CompressedTrackOffsets[],
	const uniform unsigned int8 CompressedByteStream[],
	const uniform int CompressedNumberOfFrames,
	const uniform float SequenceLength,
	const uniform float RelativePos,
	const uniform unsigned int8 InterpolationType,
	const uniform int PairCount)
{
	foreach(PairIndex = 0 ... PairCount)
	{
		#pragma ignore warning(perf)
		const BoneTrackPair Pair = DesiredPairs[PairIndex];
		const int TrackIndex = Pair.TrackIndex;
		const int AtomIndex = Pair.AtomIndex;

		const int* TrackOffsetData = CompressedTrackOffsets + (TrackIndex * 2);

		#pragma ignore warning(perf)
		const int RotKeysOffset = TrackOffsetData[1];

		FVector4f R0;

		if (RotKeysOffset != INDEX_NONE)
		{
			const unsigned int8* TrackData = CompressedByteStream + RotKeysOffset + 4;

			#pragma ignore warning(perf)
			const int Header = *((int*)(CompressedByteStream + RotKeysOffset));

			int KeyFormat;
			int NumKeys;
			int FormatFlags;
			int BytesPerKey;
			int FixedBytes;
			DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/BytesPerKey, /*OUT*/ FixedBytes);

			// Figure out the key indexes
			int Index0 = 0;
			int Index1 = 0;

			// Alpha is volatile to force the compiler to store it to memory immediately, so it is ready to be loaded into a vector register without a LHS after decompressing a track 
			float Alpha = 0.0f;

			if (NumKeys > 1)
			{
				if ((FormatFlags & 0x8) == 0)
				{
					Alpha = TimeToIndex(SequenceLength, RelativePos, NumKeys, InterpolationType, Index0, Index1);
				}
				else
				{
					const unsigned int8* FrameTable = Align(TrackData + FixedBytes + BytesPerKey * NumKeys, 4);
					Alpha = TimeToIndex(InterpolationType, CompressedNumberOfFrames, FrameTable, RelativePos, NumKeys, Index0, Index1);
				}
			}

			// Unpack the first key
			const unsigned int8* KeyData0 = TrackData + FixedBytes + (Index0 * BytesPerKey);

			DecompressRotation(KeyFormat, FormatFlags, R0, TrackData, KeyData0);

			// If there is a second key, figure out the lerp between the two of them
			if (Index0 != Index1)
			{
				const unsigned int8* KeyData1 = TrackData + FixedBytes + (Index1 * BytesPerKey);

				FVector4f R1;
				DecompressRotation(KeyFormat, FormatFlags, R1, TrackData, KeyData1);

				// Fast linear quaternion interpolation.
				R0 = QuatFastLerp(R0, R1, Alpha);
			}

			R0 = VectorNormalizeQuaternion(R0);
		}
		else
		{
			// Identity track
			R0 = Float0001;
		}

		uniform FVector4f AOSScratch[programCount];
		soa_to_aos4(R0.V[0], R0.V[1], R0.V[2], R0.V[3], (uniform float* uniform)&AOSScratch);
		foreach_active(i)
		{
			unmasked
			{
				uniform int uAtomIndex = extract(AtomIndex, i);
				uniform FVector4 *uniform BoneAtomRotation = (uniform FVector4 *uniform)&Atoms[uAtomIndex].Rotation;
				*BoneAtomRotation = ConvertVector4fTo4Native(AOSScratch[i]);
			}
		}
	}
}

export void GetPerTrackCompressionPoseTranslations(
	uniform FTransform Atoms[],
	const uniform BoneTrackPair DesiredPairs[],
	const uniform int CompressedTrackOffsets[],
	const uniform unsigned int8 CompressedByteStream[],
	const uniform int CompressedNumberOfFrames,
	const uniform float SequenceLength,
	const uniform float RelativePos,
	const uniform unsigned int8 InterpolationType,
	const uniform int PairCount)
{
	foreach(PairIndex = 0 ... PairCount)
	{
		#pragma ignore warning(perf)
		const BoneTrackPair Pair = DesiredPairs[PairIndex];
		const int TrackIndex = Pair.TrackIndex;
		const int AtomIndex = Pair.AtomIndex;

		const int* TrackOffsetData = CompressedTrackOffsets + (TrackIndex * 2);

		#pragma ignore warning(perf)
		const int PosKeysOffset = TrackOffsetData[0];

		FVector3f R0;

		if (PosKeysOffset != INDEX_NONE)
		{
			const unsigned int8* TrackData = CompressedByteStream + PosKeysOffset + 4;

			#pragma ignore warning(perf)
			const int Header = *((int*)(CompressedByteStream + PosKeysOffset));

			int KeyFormat;
			int NumKeys;
			int FormatFlags;
			int BytesPerKey;
			int FixedBytes;
			DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/BytesPerKey, /*OUT*/ FixedBytes);

			// Figure out the key indexes
			int Index0 = 0;
			int Index1 = 0;

			// Alpha is volatile to force the compiler to store it to memory immediately, so it is ready to be loaded into a vector register without a LHS after decompressing a track 
			float Alpha = 0.0f;

			if (NumKeys > 1)
			{
				if ((FormatFlags & 0x8) == 0)
				{
					Alpha = TimeToIndex(SequenceLength, RelativePos, NumKeys, InterpolationType, Index0, Index1);
				}
				else
				{
					const unsigned int8* FrameTable = Align(TrackData + FixedBytes + BytesPerKey * NumKeys, 4);
					Alpha = TimeToIndex(InterpolationType, CompressedNumberOfFrames, FrameTable, RelativePos, NumKeys, Index0, Index1);
				}
			}

			// Unpack the first key
			const unsigned int8* KeyData0 = TrackData + FixedBytes + (Index0 * BytesPerKey);

			DecompressTranslation(KeyFormat, FormatFlags, R0, TrackData, KeyData0);

			// If there is a second key, figure out the lerp between the two of them
			if (Index0 != Index1)
			{
				const unsigned int8* KeyData1 = TrackData + FixedBytes + (Index1 * BytesPerKey);

				FVector3f R1;
				DecompressTranslation(KeyFormat, FormatFlags, R1, TrackData, KeyData1);

				R0 = VectorLerp(R0, R1, Alpha);
			}
		}
		else
		{
			// Identity track
			R0 = FloatZeroVector;
		}

		uniform FVector4f AOSScratch[programCount];
		soa_to_aos4(R0.V[0], R0.V[1], R0.V[2], 0.0f, (uniform float* uniform)&AOSScratch);
		foreach_active(i)
		{
			unmasked
			{
				uniform int uAtomIndex = extract(AtomIndex, i);
				uniform FVector4 *uniform BoneAtomTranslation = (uniform FVector4 *uniform)&Atoms[uAtomIndex].Translation;
				*BoneAtomTranslation = ConvertVector4fTo4Native(AOSScratch[i]);
			}
		}
	}
}

export void GetPerTrackCompressionPoseScales(
	uniform FTransform Atoms[],
	const uniform BoneTrackPair DesiredPairs[],
	const uniform int CompressedScaleOffsets[],
	const uniform int StripSize,
	const uniform unsigned int8 CompressedByteStream[],
	const uniform int CompressedNumberOfFrames,
	const uniform float SequenceLength,
	const uniform float RelativePos,
	const uniform unsigned int8 InterpolationType,
	const uniform int PairCount)
{
	foreach(PairIndex = 0 ... PairCount)
	{
		#pragma ignore warning(perf)
		const BoneTrackPair Pair = DesiredPairs[PairIndex];
		const int TrackIndex = Pair.TrackIndex;
		const int AtomIndex = Pair.AtomIndex;

		const int* ScaleData = CompressedScaleOffsets + (TrackIndex * StripSize);

		#pragma ignore warning(perf)
		const int32 ScaleKeysOffset = ScaleData[0];

		FVector3f R0;

		if (ScaleKeysOffset != INDEX_NONE)
		{
			const unsigned int8* TrackData = CompressedByteStream + ScaleKeysOffset + 4;

			#pragma ignore warning(perf)
			const int32 Header = *((int32*)(CompressedByteStream + ScaleKeysOffset));

			int32 KeyFormat;
			int32 NumKeys;
			int32 FormatFlags;
			int32 BytesPerKey;
			int32 FixedBytes;
			DecomposeHeader(Header, /*OUT*/ KeyFormat, /*OUT*/ NumKeys, /*OUT*/ FormatFlags, /*OUT*/BytesPerKey, /*OUT*/ FixedBytes);

			// Figure out the key indexes
			int32 Index0 = 0;
			int32 Index1 = 0;

			// Alpha is volatile to force the compiler to store it to memory immediately, so it is ready to be loaded into a vector register without a LHS after decompressing a track 
			float Alpha = 0.0f;

			if (NumKeys > 1)
			{
				if ((FormatFlags & 0x8) == 0)
				{
					Alpha = TimeToIndex(SequenceLength, RelativePos, NumKeys, InterpolationType, Index0, Index1);
				}
				else
				{
					const unsigned int8* FrameTable = Align(TrackData + FixedBytes + BytesPerKey * NumKeys, 4);
					Alpha = TimeToIndex(InterpolationType, CompressedNumberOfFrames, FrameTable, RelativePos, NumKeys, Index0, Index1);
				}
			}

			// Unpack the first key
			const unsigned int8* KeyData0 = TrackData + FixedBytes + (Index0 * BytesPerKey);

			DecompressScale(KeyFormat, FormatFlags, R0, TrackData, KeyData0);

			// If there is a second key, figure out the lerp between the two of them
			if (Index0 != Index1)
			{
				const unsigned int8* KeyData1 = TrackData + FixedBytes + (Index1 * BytesPerKey);

				FVector3f R1;
				DecompressScale(KeyFormat, FormatFlags, R1, TrackData, KeyData1);

				R0 = VectorLerp(R0, R1, Alpha);
			}
		}
		else
		{
			// Identity track
			R0 = FloatZeroVector;
		}

		uniform FVector4f AOSScratch[programCount];
		soa_to_aos4(R0.V[0], R0.V[1], R0.V[2], 0.0f, (uniform float* uniform)&AOSScratch);
		foreach_active(i)
		{
			unmasked
			{
				uniform int uAtomIndex = extract(AtomIndex, i);
				uniform FVector4 *uniform BoneAtomScale = (uniform FVector4 *uniform)&Atoms[uAtomIndex].Scale3D;
				*BoneAtomScale = ConvertVector4fTo4Native(AOSScratch[i]);
			}
		}
	}
}
