// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimEncoding_VariableKeyLerp.cpp: Skeletal mesh animation functions.
=============================================================================*/ 

#include "AnimEncoding_VariableKeyLerp.h"
#if INTEL_ISPC
#include "AnimEncoding_VariableKeyLerp.ispc.generated.h"
#include "AutoRTFM.h"

static_assert(sizeof(ispc::FTransform) == sizeof(FTransform), "sizeof(ispc::FTransform) != sizeof(FTransform)");
static_assert(sizeof(ispc::BoneTrackPair) == sizeof(BoneTrackPair), "sizeof(ispc::BoneTrackPair) != sizeof(BoneTrackPair)");
#endif

#if !defined(ANIM_VARIABLE_KEY_LERP_ISPC_ENABLED_DEFAULT)
#define ANIM_VARIABLE_KEY_LERP_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bAnim_VariableKeyLerp_ISPC_Enabled = INTEL_ISPC && ANIM_VARIABLE_KEY_LERP_ISPC_ENABLED_DEFAULT;
#else
#include "HAL/IConsoleManager.h"
static bool bAnim_VariableKeyLerp_ISPC_Enabled = ANIM_VARIABLE_KEY_LERP_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarAnimVariableKeyLerpISPCEnabled(TEXT("a.VariableKeyLerp.ISPC"), bAnim_VariableKeyLerp_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in variable key anim encoding"));
#endif

/**
 * Handles the ByteSwap of compressed rotation data on import
 *
 * @param	CompressedData		The compressed animation data being operated on.
 * @param	MemoryReader	The FMemoryReader to read from.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys present in the stream.
 */
void AEFVariableKeyLerpShared::ByteSwapRotationIn(
	FUECompressedAnimData& CompressedData,
	FMemoryReader& MemoryReader,
	uint8*& TrackData,
	int32 NumKeys)
{
	AEFConstantKeyLerpShared::ByteSwapRotationIn(CompressedData, MemoryReader, TrackData, NumKeys);

	// Load the track table if present
	if (NumKeys > 1)
	{
		// Like the compressed byte stream, pad the serialization stream to four bytes.
		// As a sanity check, each pad byte can be checked to be the PadSentinel.
		PadMemoryReader(&MemoryReader, TrackData, 4); 

		// swap the track table
		const size_t EntryStride = (CompressedData.CompressedNumberOfKeys > 0xFF) ? sizeof(uint16) : sizeof(uint8);
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, EntryStride);
		}
	}
}

/**
 * Handles the ByteSwap of compressed translation data on import
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryReader	The FMemoryReader to read from.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys present in the stream.
 */
void AEFVariableKeyLerpShared::ByteSwapTranslationIn(
	FUECompressedAnimData& CompressedData,
	FMemoryReader& MemoryReader,
	uint8*& TrackData,
	int32 NumKeys)
{
	AEFConstantKeyLerpShared::ByteSwapTranslationIn(CompressedData, MemoryReader, TrackData, NumKeys);

	// Load the track table if present
	if (NumKeys > 1)
	{
		// Like the compressed byte stream, pad the serialization stream to four bytes.
		// As a sanity check, each pad byte can be checked to be the PadSentinel.
		PadMemoryReader(&MemoryReader, TrackData, 4); 

		// swap the track table
		const size_t EntryStride = (CompressedData.CompressedNumberOfKeys > 0xFF) ? sizeof(uint16) : sizeof(uint8);
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, EntryStride);
		}
	}
}

/**
 * Handles the ByteSwap of compressed Scale data on import
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryReader	The FMemoryReader to read from.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys present in the stream.
 */
void AEFVariableKeyLerpShared::ByteSwapScaleIn(
	FUECompressedAnimData& CompressedData,
	FMemoryReader& MemoryReader,
	uint8*& TrackData,
	int32 NumKeys)
{
	AEFConstantKeyLerpShared::ByteSwapScaleIn(CompressedData, MemoryReader, TrackData, NumKeys);

	// Load the track table if present
	if (NumKeys > 1)
	{
		// Like the compressed byte stream, pad the serialization stream to four bytes.
		// As a sanity check, each pad byte can be checked to be the PadSentinel.
		PadMemoryReader(&MemoryReader, TrackData, 4); 

		// swap the track table
		const size_t EntryStride = (CompressedData.CompressedNumberOfKeys > 0xFF) ? sizeof(uint16) : sizeof(uint8);
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			AC_UnalignedSwap(MemoryReader, TrackData, EntryStride);
		}
	}
}
/**
 * Handles the ByteSwap of compressed rotation data on export
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryWriter	The FMemoryWriter to write to.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys to write to the stream.
 */
void AEFVariableKeyLerpShared::ByteSwapRotationOut(
	FUECompressedAnimData& CompressedData,
	FMemoryWriter& MemoryWriter,
	uint8*& TrackData,
	int32 NumKeys)
{
	AEFConstantKeyLerpShared::ByteSwapRotationOut(CompressedData, MemoryWriter, TrackData, NumKeys);

	// Store the track table if needed
	if (NumKeys > 1)
	{
		// Like the compressed byte stream, pad the serialization stream to four bytes.
		// As a sanity check, each pad byte can be checked to be the PadSentinel.
		PadMemoryWriter(&MemoryWriter, TrackData, 4);

		// swap the track table
		const size_t EntryStride = (CompressedData.CompressedNumberOfKeys > 0xFF) ? sizeof(uint16) : sizeof(uint8);
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, EntryStride);
		}
	}
}

/**
 * Handles the ByteSwap of compressed translation data on export
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryWriter	The FMemoryWriter to write to.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys to write to the stream.
 */
void AEFVariableKeyLerpShared::ByteSwapTranslationOut(
	FUECompressedAnimData& CompressedData,
	FMemoryWriter& MemoryWriter,
	uint8*& TrackData,
	int32 NumKeys)
{
	AEFConstantKeyLerpShared::ByteSwapTranslationOut(CompressedData, MemoryWriter, TrackData, NumKeys);

	// Store the track table if needed
	if (NumKeys > 1)
	{
		// Like the compressed byte stream, pad the serialization stream to four bytes.
		// As a sanity check, each pad byte can be checked to be the PadSentinel.
		PadMemoryWriter(&MemoryWriter, TrackData, 4);

		// swap the track table
		const size_t EntryStride = (CompressedData.CompressedNumberOfKeys > 0xFF) ? sizeof(uint16) : sizeof(uint8);
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, EntryStride);
		}
	}
}


/**
 * Handles the ByteSwap of compressed Scale data on export
 *
 * @param	CompressedData	The compressed animation data being operated on.
 * @param	MemoryWriter	The FMemoryWriter to write to.
 * @param	TrackData		The compressed data stream.
 * @param	NumKeys			The number of keys to write to the stream.
 */
void AEFVariableKeyLerpShared::ByteSwapScaleOut(
	FUECompressedAnimData& CompressedData,
	FMemoryWriter& MemoryWriter,
	uint8*& TrackData,
	int32 NumKeys)
{
	AEFConstantKeyLerpShared::ByteSwapScaleOut(CompressedData, MemoryWriter, TrackData, NumKeys);

	// Store the track table if needed
	if (NumKeys > 1)
	{
		// Like the compressed byte stream, pad the serialization stream to four bytes.
		// As a sanity check, each pad byte can be checked to be the PadSentinel.
		PadMemoryWriter(&MemoryWriter, TrackData, 4);

		// swap the track table
		const size_t EntryStride = (CompressedData.CompressedNumberOfKeys > 0xFF) ? sizeof(uint16) : sizeof(uint8);
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			AC_UnalignedSwap(MemoryWriter, TrackData, EntryStride);
		}
	}
}

#if USE_ANIMATION_CODEC_BATCH_SOLVER

/**
 * Decompress all requested rotation components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
template<int32 FORMAT>
void AEFVariableKeyLerp<FORMAT>::GetPoseRotations(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();
	
	if (PairCount == 0)
	{
		return;
	}

	if (bAnim_VariableKeyLerp_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetVariableKeyLerpPoseRotations(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				AnimData.CompressedTrackOffsets.GetData(),
				AnimData.CompressedByteStream.GetData(),
				AnimData.CompressedNumberOfKeys,
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				FORMAT,
				PairCount);
		};
#endif
	}
	else
	{
		for (int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex)
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];

			// call the decoder directly (not through the vtable)
			AEFVariableKeyLerp<FORMAT>::GetBoneAtomRotation(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

/**
 * Decompress all requested translation components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
template<int32 FORMAT>
void AEFVariableKeyLerp<FORMAT>::GetPoseTranslations(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();
	
	if (PairCount == 0)
	{
		return;
	}

	if (bAnim_VariableKeyLerp_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetVariableKeyLerpPoseTranslations(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				AnimData.CompressedTrackOffsets.GetData(),
				AnimData.CompressedByteStream.GetData(),
				AnimData.CompressedNumberOfKeys,
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				FORMAT,
				PairCount);
		};
#endif
	}
	else
	{
		for (int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex)
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];
	
			// call the decoder directly (not through the vtable)
			AEFVariableKeyLerp<FORMAT>::GetBoneAtomTranslation(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

/**
 * Decompress all requested Scale components from an Animation Sequence
 *
 * @param	Atoms			The FTransform array to fill in.
 * @param	DesiredPairs	Array of requested bone information
 * @param	DecompContext	The decompression context to use.
 */
template<int32 FORMAT>
void AEFVariableKeyLerp<FORMAT>::GetPoseScales(
	TArrayView<FTransform>& Atoms,
	const BoneTrackArray& DesiredPairs,
	FAnimSequenceDecompressionContext& DecompContext)
{
	const int32 PairCount = DesiredPairs.Num();

	if (PairCount == 0)
	{
		return;
	}

	if (bAnim_VariableKeyLerp_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

			const TArrayView<int32> ScaleOffsets = AnimData.CompressedScaleOffsets.OffsetData;
			const int32 StripSize = AnimData.CompressedScaleOffsets.StripSize;

			AutoRTFM::RecordOpenWrite(
				Atoms.GetData(),
				Atoms.NumBytes());
			ispc::GetVariableKeyLerpPoseScales(
				reinterpret_cast<ispc::FTransform*>(Atoms.GetData()),
				reinterpret_cast<const ispc::BoneTrackPair*>(DesiredPairs.GetData()),
				ScaleOffsets.GetData(),
				StripSize,
				AnimData.CompressedByteStream.GetData(),
				AnimData.CompressedNumberOfKeys,
				DecompContext.GetRelativePosition(),
				(uint8)DecompContext.Interpolation,
				FORMAT,
				PairCount);
		};
#endif
	}
	else
	{
		for (int32 PairIndex = 0; PairIndex < PairCount; ++PairIndex)
		{
			const BoneTrackPair& Pair = DesiredPairs[PairIndex];
			const int32 TrackIndex = Pair.TrackIndex;
			const int32 AtomIndex = Pair.AtomIndex;
			FTransform& BoneAtom = Atoms[AtomIndex];
	
			// call the decoder directly (not through the vtable)
			AEFVariableKeyLerp<FORMAT>::GetBoneAtomScale(BoneAtom, DecompContext, TrackIndex);
		}
	}
}

template class AEFVariableKeyLerp<ACF_None>;
template class AEFVariableKeyLerp<ACF_Float96NoW>;
template class AEFVariableKeyLerp<ACF_Fixed48NoW>;
template class AEFVariableKeyLerp<ACF_IntervalFixed32NoW>;
template class AEFVariableKeyLerp<ACF_Fixed32NoW>;
template class AEFVariableKeyLerp<ACF_Float32NoW>;
template class AEFVariableKeyLerp<ACF_Identity>;

#endif
