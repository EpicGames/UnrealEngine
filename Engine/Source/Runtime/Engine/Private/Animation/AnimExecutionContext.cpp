// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNodeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimExecutionContext)

FAnimExecutionContext::FData::FData(const FAnimationBaseContext& InContext)
{
	Context = const_cast<FAnimationBaseContext*>(&InContext);
	ContextType = EContextType::Base;
}

FAnimExecutionContext::FData::FData(const FAnimationInitializeContext& InContext)
{
	Context = const_cast<FAnimationInitializeContext*>(&InContext);
	ContextType = EContextType::Initialize;
}

FAnimExecutionContext::FData::FData(const FAnimationUpdateContext& InContext)
{
	Context = const_cast<FAnimationUpdateContext*>(&InContext);
	ContextType = EContextType::Update;
}

FAnimExecutionContext::FData::FData(FPoseContext& InContext)
{
	Context = &InContext;
	ContextType = EContextType::Pose;
}

FAnimExecutionContext::FData::FData(FComponentSpacePoseContext& InContext)
{
	Context = &InContext;
	ContextType = EContextType::ComponentSpacePose;
}

FAnimationInitializeContext* FAnimInitializationContext::GetContext() const
{
	return GetInternalContext<FAnimInitializationContext, FAnimationInitializeContext>();
}

FAnimationUpdateContext* FAnimUpdateContext::GetContext() const
{
	return GetInternalContext<FAnimUpdateContext, FAnimationUpdateContext>();
}

FPoseContext* FAnimPoseContext::GetContext() const
{
	return GetInternalContext<FAnimPoseContext, FPoseContext>();
}

FComponentSpacePoseContext* FAnimComponentSpacePoseContext::GetContext() const
{
	return GetInternalContext<FAnimPoseContext, FComponentSpacePoseContext>();
}
