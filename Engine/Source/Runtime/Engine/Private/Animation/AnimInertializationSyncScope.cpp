// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimInertializationSyncScope.h"

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::Anim::FAnimInertializationSyncScope);
