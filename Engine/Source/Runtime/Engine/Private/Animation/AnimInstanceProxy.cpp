// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimMontageEvaluationState.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimSlotEvaluationPose.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimTrace.h"
#include "Animation/PoseAsset.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/AnimStats.h"
#include "Logging/MessageLog.h"
#include "Animation/AnimNode_AssetPlayerBase.h"
#include "Animation/AnimNode_StateMachine.h"
#include "Animation/AnimNode_TransitionResult.h"
#include "Animation/AnimNode_SaveCachedPose.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "Animation/AnimSubsystemInstance.h"
#include "Animation/BlendProfile.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/BlendProfileScratchData.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Engine.h"
#include "Engine/SkeletalMesh.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/WorldSettings.h"
#include "Animation/AnimNode_Root.h"
#include "Animation/AnimNode_LinkedAnimLayer.h"
#include "Animation/AnimSyncScope.h"
#include "Animation/AnimNotifyStateMachineInspectionLibrary.h"
#include "Animation/MirrorDataTable.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#if WITH_EDITOR
#include "Engine/PoseWatch.h"
#endif

#define DO_ANIMSTAT_PROCESSING(StatName) DEFINE_STAT(STAT_ ## StatName)
#include "Animation/AnimMTStats.h" // IWYU pragma: keep
#undef DO_ANIMSTAT_PROCESSING

#define DO_ANIMSTAT_PROCESSING(StatName) DEFINE_STAT(STAT_ ## StatName ## _WorkerThread)
#include "Animation/AnimMTStats.h" // IWYU pragma: keep

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimInstanceProxy)

#undef DO_ANIMSTAT_PROCESSING

LLM_DEFINE_TAG(Animation_Graph);

#define LOCTEXT_NAMESPACE "AnimInstance"

const FName NAME_AnimBlueprintLog("AnimBlueprintLog");
const FName NAME_PIELog("PIE");
const FName NAME_Evaluate("Evaluate");
const FName NAME_Update("Update");
const FName NAME_AnimGraph("AnimGraph");

bool GWarnOnDuplicateSlotNode = true;
static FAutoConsoleVariableRef CVarWarnOnDuplicateSlotNode(
	TEXT("a.WarnOnDuplicateSlotNode"),
	GWarnOnDuplicateSlotNode,
	TEXT("1 = Warn when duplicate slot nodes exist in the anim graph. 0 = off"));

FAnimInstanceProxy::FAnimInstanceProxy()
	: AnimInstanceObject(nullptr)
	, AnimClassInterface(nullptr)
	, Skeleton(nullptr)
	, SkeletalMeshComponent(nullptr)
	, MainInstanceProxy(nullptr)
	, CurrentDeltaSeconds(0.0f)
	, CurrentTimeDilation(1.0f)
	, RootNode(nullptr)
	, DefaultLinkedInstanceInputNode(nullptr)
	, BufferWriteIndex(0)
	, RootMotionMode(ERootMotionMode::NoRootMotionExtraction)
	, FrameCounterForUpdate(0)
	, FrameCounterForNodeUpdate(0)
	, RequiredBones(MakeShared<FBoneContainer>())	// We sometime query for this before RecalcRequiredBones has been called
	, CacheBonesRecursionCounter(0)
	, bUpdatingRoot(false)
	, bBoneCachesInvalidated(false)
	, bShouldExtractRootMotion(false)
	, bDeferRootNodeInitialization(false)
#if WITH_EDITORONLY_DATA
	, bIsBeingDebugged(false)
	, bIsGameWorld(false)
#endif
	, bInitializeSubsystems(false)
	, bUseMainInstanceMontageEvaluationData(false)
{
}

FAnimInstanceProxy::FAnimInstanceProxy(UAnimInstance* Instance)
	: AnimInstanceObject(Instance)
	, AnimClassInterface(IAnimClassInterface::GetFromClass(Instance->GetClass()))
	, Skeleton(nullptr)
	, SkeletalMeshComponent(nullptr)
	, MainInstanceProxy(nullptr)
	, CurrentDeltaSeconds(0.0f)
	, CurrentTimeDilation(1.0f)
	, RootNode(nullptr)
	, DefaultLinkedInstanceInputNode(nullptr)
	, BufferWriteIndex(0)
	, RootMotionMode(ERootMotionMode::NoRootMotionExtraction)
	, FrameCounterForUpdate(0)
	, FrameCounterForNodeUpdate(0)
	, RequiredBones(MakeShared<FBoneContainer>())	// We sometime query for this before RecalcRequiredBones has been called
	, CacheBonesRecursionCounter(0)
	, bUpdatingRoot(false)
	, bBoneCachesInvalidated(false)
	, bShouldExtractRootMotion(false)
	, bDeferRootNodeInitialization(false)
#if WITH_EDITORONLY_DATA
	, bIsBeingDebugged(false)
#endif
	, bInitializeSubsystems(false)
	, bUseMainInstanceMontageEvaluationData(false)
{
}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FAnimInstanceProxy::FAnimInstanceProxy(const FAnimInstanceProxy&) = default;
FAnimInstanceProxy& FAnimInstanceProxy::operator=(FAnimInstanceProxy&&) = default;
FAnimInstanceProxy& FAnimInstanceProxy::operator=(const FAnimInstanceProxy&) = default;
FAnimInstanceProxy::~FAnimInstanceProxy() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FAnimInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	UpdateAnimationNode_WithRoot(InContext, RootNode, NAME_AnimGraph);
}

void FAnimInstanceProxy::UpdateAnimationNode_WithRoot(const FAnimationUpdateContext& InContext, FAnimNode_Base* InRootNode, FName InLayerName)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/Graph"));

	TRACE_SCOPED_ANIM_GRAPH(InContext)
	TRACE_SCOPED_ANIM_NODE(InContext)
	
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	if(InRootNode != nullptr)
	{
		if (InRootNode == RootNode)
		{
			UpdateCounter.Increment();
		}

		UE::Anim::FNodeFunctionCaller::InitialUpdate(InContext, *InRootNode);
		UE::Anim::FNodeFunctionCaller::BecomeRelevant(InContext, *InRootNode);
		UE::Anim::FNodeFunctionCaller::Update(InContext, *InRootNode);
		InRootNode->Update_AnyThread(InContext);

		// We've updated the graph, now update the fractured saved pose sections
		// Note SavedPoseQueueMap can be empty if we have no cached poses in use
		if(TArray<FAnimNode_SaveCachedPose*>* SavedPoseQueue = SavedPoseQueueMap.Find(InLayerName))
		{
			for(FAnimNode_SaveCachedPose* PoseNode : *SavedPoseQueue)
			{
				PoseNode->PostGraphUpdate();
			}
		}
	}
}

void FAnimInstanceProxy::AddReferencedObjects(UAnimInstance* InAnimInstance, FReferenceCollector& Collector)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	
	Sync.AddReferencedObjects(InAnimInstance, Collector);
}

void FAnimInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// copy anim instance object if it has not already been set up
	AnimInstanceObject = InAnimInstance;

	AnimClassInterface = IAnimClassInterface::GetFromClass(InAnimInstance->GetClass());

	InitializeObjects(InAnimInstance);

	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();

		// Grab a pointer to the default root node, if any
		RootNode = nullptr;
		if(AnimClassInterface->GetAnimBlueprintFunctions().Num() > 0)
		{
			if(FProperty* RootNodeProperty = AnimClassInterface->GetAnimBlueprintFunctions()[0].OutputPoseNodeProperty)
			{
				RootNode = RootNodeProperty->ContainerPtrToValuePtr<FAnimNode_Root>(InAnimInstance);
			}
		}

		// Initialise the pose node list
		const TMap<FName, FCachedPoseIndices>& PoseNodeIndicesMap = AnimClassInterface->GetOrderedSavedPoseNodeIndicesMap();
		SavedPoseQueueMap.Reset();
		for(const TPair<FName, FCachedPoseIndices>& Pair : PoseNodeIndicesMap)
		{
			TArray<FAnimNode_SaveCachedPose*>& SavedPoseQueue = SavedPoseQueueMap.Add(Pair.Key);
			for(int32 Idx : Pair.Value.OrderedSavedPoseNodeIndices)
			{
				int32 ActualPropertyIdx = AnimNodeProperties.Num() - 1 - Idx;
				FAnimNode_SaveCachedPose* ActualPoseNode = AnimNodeProperties[ActualPropertyIdx]->ContainerPtrToValuePtr<FAnimNode_SaveCachedPose>(InAnimInstance);
				SavedPoseQueue.Add(ActualPoseNode);
			}
		}

		// if no mesh, use Blueprint Skeleton
		if (Skeleton == nullptr)
		{
			Skeleton = AnimClassInterface->GetTargetSkeleton();
		}

		// Initialize state buffers
		int32 NumStates = 0;
		const TArray<FBakedAnimationStateMachine>& BakedMachines = AnimClassInterface->GetBakedStateMachines();
		const int32 NumMachines = BakedMachines.Num();
		for(int32 MachineClassIndex = 0; MachineClassIndex < NumMachines; ++MachineClassIndex)
		{
			const FBakedAnimationStateMachine& Machine = BakedMachines[MachineClassIndex];
			StateMachineClassIndexToWeightOffset.Add(MachineClassIndex, NumStates);
			NumStates += Machine.States.Num();
		}
		StateWeightArrays[0].Reset(NumStates);
		StateWeightArrays[0].AddZeroed(NumStates);
		StateWeightArrays[1].Reset(NumStates);
		StateWeightArrays[1].AddZeroed(NumStates);

		MachineWeightArrays[0].Reset(NumMachines);
		MachineWeightArrays[0].AddZeroed(NumMachines);
		MachineWeightArrays[1].Reset(NumMachines);
		MachineWeightArrays[1].AddZeroed(NumMachines);

#if WITH_EDITORONLY_DATA
		if (UAnimBlueprint* Blueprint = Cast<UAnimBlueprint>(InAnimInstance->GetClass()->ClassGeneratedBy))
		{
			// If our blueprint is in an error or dirty state upon initialization, anim graph shouldn't run
			if ((Blueprint->Status == BS_Error) || (Blueprint->Status == BS_Dirty))
			{
				RootNode = nullptr;
			}
		}
#endif
	}
	else
	{
		RootNode = (FAnimNode_Base*) GetCustomRootNode();
	}

#if ENABLE_ANIM_LOGGING
	ActorName = GetNameSafe(InAnimInstance->GetOwningActor());
#endif

	UpdateCounter.Reset();
	ReinitializeSlotNodes();

	if (const USkeletalMeshComponent* SkelMeshComp = InAnimInstance->GetOwningComponent())
	{
		ComponentTransform = SkelMeshComp->GetComponentTransform();
		ComponentRelativeTransform = SkeletalMeshComponent->GetRelativeTransform();

		const AActor* OwningActor = SkeletalMeshComponent->GetOwner();
		ActorTransform = OwningActor ? OwningActor->GetActorTransform() : FTransform::Identity;
	}
	else
	{
		ComponentTransform = FTransform::Identity;
		ComponentRelativeTransform = FTransform::Identity;
		ActorTransform = FTransform::Identity;
	}
}

void FAnimInstanceProxy::InitializeCachedClassData()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	LODDisabledGameThreadPreUpdateNodes.Reset();
	GameThreadPreUpdateNodes.Reset();
	DynamicResetNodes.Reset();

	if(AnimClassInterface)
	{
		// cache any state machine descriptions we have
		for (const FStructProperty* Property : AnimClassInterface->GetStateMachineNodeProperties())
		{
			FAnimNode_StateMachine* StateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
			StateMachine->CacheMachineDescription(AnimClassInterface);
		}
		
		// Cache any preupdate nodes
		for (const FStructProperty* Property : AnimClassInterface->GetPreUpdateNodeProperties())
		{
			FAnimNode_Base* AnimNode = Property->ContainerPtrToValuePtr<FAnimNode_Base>(AnimInstanceObject);
			GameThreadPreUpdateNodes.Add(AnimNode);
		}

		// Cache any dynamic reset nodes
		for (const FStructProperty* Property : AnimClassInterface->GetDynamicResetNodeProperties())
		{
			FAnimNode_Base* AnimNode = Property->ContainerPtrToValuePtr<FAnimNode_Base>(AnimInstanceObject);
			DynamicResetNodes.Add(AnimNode);
		}

		// Cache default linked input pose 
		for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
		{
			if(AnimBlueprintFunction.Name == NAME_AnimGraph)
			{
				check(AnimBlueprintFunction.InputPoseNames.Num() == AnimBlueprintFunction.InputPoseNodeProperties.Num());
				for(int32 InputIndex = 0; InputIndex < AnimBlueprintFunction.InputPoseNames.Num(); ++InputIndex)
				{
					if(AnimBlueprintFunction.InputPoseNames[InputIndex] == FAnimNode_LinkedInputPose::DefaultInputPoseName && AnimBlueprintFunction.InputPoseNodeProperties[InputIndex] != nullptr)
					{
						DefaultLinkedInstanceInputNode = AnimBlueprintFunction.InputPoseNodeProperties[InputIndex]->ContainerPtrToValuePtr<FAnimNode_LinkedInputPose>(CastChecked<UAnimInstance>(GetAnimInstanceObject()));
						break;
					}
				}
			}
		}
	}
}

void FAnimInstanceProxy::InitializeRootNode(bool bInDeferRootNodeInitialization)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	InitializeCachedClassData();
	
	if(AnimClassInterface)
	{
		// Init any nodes that need non-relevancy based initialization
		UAnimInstance* AnimInstance = CastChecked<UAnimInstance>(GetAnimInstanceObject());
		for (const FStructProperty* Property : AnimClassInterface->GetInitializationNodeProperties())
		{
			FAnimNode_Base* AnimNode = Property->ContainerPtrToValuePtr<FAnimNode_Base>(AnimInstanceObject);
			AnimNode->OnInitializeAnimInstance(this, AnimInstance);
		}
	}
	else
	{
		auto InitializeNode = [this](FAnimNode_Base* AnimNode)
		{
			if(AnimNode->NeedsOnInitializeAnimInstance())
			{
				AnimNode->OnInitializeAnimInstance(this, CastChecked<UAnimInstance>(GetAnimInstanceObject()));
			}

			if (AnimNode->HasPreUpdate())
			{
				GameThreadPreUpdateNodes.Add(AnimNode);
			}

			if (AnimNode->NeedsDynamicReset())
			{
				DynamicResetNodes.Add(AnimNode);
			}
		};

		//We have a custom root node, so get the associated nodes and initialize them
		TArray<FAnimNode_Base*> CustomNodes;
		GetCustomNodes(CustomNodes);
		for(FAnimNode_Base* Node : CustomNodes)
		{
			if(Node)
			{
				InitializeNode(Node);
			}
		}
	}

	if(!bInDeferRootNodeInitialization)
	{
		InitializeRootNode_WithRoot(RootNode);
	}
	else
	{
		bDeferRootNodeInitialization = true;
	}

	bInitializeSubsystems = true;
}

void FAnimInstanceProxy::InitializeRootNode_WithRoot(FAnimNode_Base* InRootNode)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/Graph"));

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (InRootNode != nullptr)
	{
		FAnimationUpdateSharedContext SharedContext;
		FAnimationInitializeContext InitContext(this, &SharedContext);
		
		if(InRootNode == RootNode)
		{
			InitializationCounter.Increment();

			TRACE_SCOPED_ANIM_GRAPH(InitContext);

			InRootNode->Initialize_AnyThread(InitContext);
		}
		else
		{
			InRootNode->Initialize_AnyThread(InitContext);
		}
	}
}

FGuid MakeGuidForMessage(const FText& Message)
{
	FString MessageString = Message.ToString();
	const TArray<TCHAR, FString::AllocatorType> CharArray = MessageString.GetCharArray();

	FSHA1 Sha;

	Sha.Update((uint8*)CharArray.GetData(), CharArray.Num() * CharArray.GetTypeSize());

	Sha.Final();

	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	return FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
}

FName FAnimInstanceProxy::GetTargetLogNameForCurrentWorldType() const
{
#if WITH_EDITORONLY_DATA
	return bIsGameWorld ? NAME_PIELog : NAME_AnimBlueprintLog;
#else
	return NAME_AnimBlueprintLog;
#endif
}

void FAnimInstanceProxy::LogMessage(FName InLogType, const TSharedRef<FTokenizedMessage>& InMessage) const
{
#if ENABLE_ANIM_LOGGING
	FGuid CurrentMessageGuid = MakeGuidForMessage(InMessage->ToText());
	if(!PreviouslyLoggedMessages.Contains(CurrentMessageGuid))
	{
		PreviouslyLoggedMessages.Add(CurrentMessageGuid);
		if (TArray<FLogMessageEntry>* LoggedMessages = LoggedMessagesMap.Find(InLogType))
		{
			LoggedMessages->Emplace(InMessage);
		}
	}
#endif
}

void FAnimInstanceProxy::LogMessage(FName InLogType, EMessageSeverity::Type InSeverity, const FText& InMessage) const
{
	LogMessage(InLogType, FTokenizedMessage::Create(InSeverity, InMessage));
}

void FAnimInstanceProxy::Uninitialize(UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	MontageEvaluationData.Reset();
	SlotGroupInertializationRequestDataMap.Reset();
	DefaultLinkedInstanceInputNode = nullptr;
	ResetAnimationCurves();
	MaterialParametersToClear.Reset();
	ActiveAnimNotifiesSinceLastTick.Reset();
	Sync.ResetAll();
}

void FAnimInstanceProxy::UpdateActiveAnimNotifiesSinceLastTick(const FAnimNotifyQueue& AnimInstanceQueue)
{
	ActiveAnimNotifiesSinceLastTick.Reset(AnimInstanceQueue.AnimNotifies.Num());
	ActiveAnimNotifiesSinceLastTick.Append(AnimInstanceQueue.AnimNotifies);
}

void FAnimInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	InitializeObjects(InAnimInstance);

	UWorld* World = SkeletalMeshComponent ? SkeletalMeshComponent->GetWorld() : nullptr;
	AWorldSettings* WorldSettings = World ? World->GetWorldSettings() : nullptr;

	CurrentDeltaSeconds = DeltaSeconds;
	CurrentTimeDilation = WorldSettings ? WorldSettings->GetEffectiveTimeDilation() : 1.0f;
	RootMotionMode = InAnimInstance->RootMotionMode;
	bShouldExtractRootMotion = InAnimInstance->ShouldExtractRootMotion();

#if WITH_EDITORONLY_DATA
	bIsGameWorld = World ? World->IsGameWorld() : false;

	UpdatedNodesThisFrame.Reset();
	NodeInputAttributesThisFrame.Reset();
	NodeOutputAttributesThisFrame.Reset();
	NodeSyncsThisFrame.Reset();

	if (FAnimBlueprintDebugData* DebugData = GetAnimBlueprintDebugData())
	{
		DebugData->ResetNodeVisitSites();
	}
#endif

	if (SkeletalMeshComponent)
	{
		// Save off LOD level that we're currently using.
		const int32 PreviousLODLevel = LODLevel;
		LODLevel = InAnimInstance->GetLODLevel();
		if (LODLevel != PreviousLODLevel)
		{
			OnPreUpdateLODChanged(PreviousLODLevel, LODLevel);
		}
	}

	NotifyQueue.Reset(SkeletalMeshComponent);

#if ENABLE_ANIM_DRAW_DEBUG
	QueuedDrawDebugItems.Reset();
#endif

#if ENABLE_ANIM_LOGGING
	//Reset logged update messages
	LoggedMessagesMap.FindOrAdd(NAME_Update).Reset();
#endif

	ClearSlotNodeWeights();

	// Reset the synchronizer
	Sync.Reset();

	TArray<float>& StateWeights = StateWeightArrays[GetBufferWriteIndex()];
	FMemory::Memset(StateWeights.GetData(), 0, StateWeights.Num() * sizeof(float));

	TArray<float>& MachineWeights = MachineWeightArrays[GetBufferWriteIndex()];
	FMemory::Memset(MachineWeights.GetData(), 0, MachineWeights.Num() * sizeof(float));

#if WITH_EDITORONLY_DATA
	UAnimBlueprint* AnimBP = GetAnimBlueprint();
	bIsBeingDebugged = AnimBP ? AnimBP->IsObjectBeingDebugged(InAnimInstance) : false;
	if (bIsBeingDebugged)
	{
		FAnimBlueprintDebugData* DebugData = AnimBP->GetDebugData();
		if (DebugData)
		{
			DebugData->DisableAllPoseWatches();
		}
	}
#endif

	ComponentTransform = SkeletalMeshComponent->GetComponentTransform();
	ComponentRelativeTransform = SkeletalMeshComponent->GetRelativeTransform();
	ActorTransform = SkeletalMeshComponent->GetOwner() ? SkeletalMeshComponent->GetOwner()->GetActorTransform() : FTransform::Identity;

	// run preupdate calls
	for (FAnimNode_Base* Node : GameThreadPreUpdateNodes)
	{
		Node->PreUpdate(InAnimInstance);
	}
}

void FAnimInstanceProxy::OnPreUpdateLODChanged(const int32 PreviousLODIndex, const int32 NewLODIndex)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// Decrease detail, see which nodes need to be disabled.
	if (NewLODIndex > PreviousLODIndex)
	{
		// Calling PreUpdate on GameThreadPreUpdateNodes is expensive, it triggers a cache miss.
		// So remove nodes from this array if they're going to get culled by LOD.
		for (int32 NodeIndex = 0; NodeIndex < GameThreadPreUpdateNodes.Num(); NodeIndex++)
		{
			FAnimNode_Base* AnimNodePtr = static_cast<FAnimNode_Base*>(GameThreadPreUpdateNodes[NodeIndex]);
			if (AnimNodePtr)
			{
				if (!AnimNodePtr->IsLODEnabled(this))
				{
					LODDisabledGameThreadPreUpdateNodes.Add(AnimNodePtr);
					GameThreadPreUpdateNodes.RemoveAt(NodeIndex, EAllowShrinking::No);
					NodeIndex--;
				}
			}
		}
	}
	// Increase detail, see which nodes need to be enabled.
	else
	{
		for (int32 NodeIndex = 0; NodeIndex < LODDisabledGameThreadPreUpdateNodes.Num(); NodeIndex++)
		{
			FAnimNode_Base* AnimNodePtr = static_cast<FAnimNode_Base*>(LODDisabledGameThreadPreUpdateNodes[NodeIndex]);
			if (AnimNodePtr)
			{
				if (AnimNodePtr->IsLODEnabled(this))
				{
					GameThreadPreUpdateNodes.Add(AnimNodePtr);
					LODDisabledGameThreadPreUpdateNodes.RemoveAt(NodeIndex, EAllowShrinking::No);
					NodeIndex--;
				}
			}
		}
	}
}

void FAnimInstanceProxy::SavePoseSnapshot(USkeletalMeshComponent* InSkeletalMeshComponent, FName SnapshotName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FPoseSnapshot& PoseSnapshot = AddPoseSnapshot(SnapshotName);
	InSkeletalMeshComponent->SnapshotPose(PoseSnapshot);
}

void FAnimInstanceProxy::PostUpdate(UAnimInstance* InAnimInstance) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

#if WITH_EDITORONLY_DATA
	if (FAnimBlueprintDebugData* DebugData = GetAnimBlueprintDebugData())
	{
		DebugData->RecordNodeVisitArray(UpdatedNodesThisFrame);
		DebugData->RecordNodeSyncsArray(NodeSyncsThisFrame);
	}
#endif

#if WITH_EDITORONLY_DATA
	if (bIsBeingDebugged)
	{
		FAnimBlueprintDebugData* DebugData = GetAnimBlueprintDebugData();
		if (DebugData)
		{
			for (const FAnimNodePoseWatch& PoseWatch : DebugData->AnimNodePoseWatch)
			{
				if (!PoseWatch.PoseWatchPoseElement->GetIsEnabled())
				{
					TRACE_ANIM_POSE_WATCH(*this, PoseWatch.PoseWatchPoseElement, PoseWatch.NodeID, TArray<FTransform>(), FBlendedHeapCurve(), TArray<uint16>(), FTransform::Identity, false);
				}
			}
		}
	}
#endif
	
	// Copy slot information to main instance if we are using the main instance's montage evaluation data.
	// Note that linked anim instance's proxies PostUpdate() will be called before the main instance's proxy PostUpdate().
	if (bUseMainInstanceMontageEvaluationData && GetMainInstanceProxy() && GetMainInstanceProxy() != this)
	{
		FAnimInstanceProxy& MainProxy = *GetMainInstanceProxy();
		
		for (const TTuple<FName, int> & LinkedSlotTrackerPair : SlotNameToTrackerIndex)
		{
			const int* MainTrackerIndexPtr = MainProxy.SlotNameToTrackerIndex.Find(LinkedSlotTrackerPair.Key);

			// Ensure slot tracker exists for main instance.
			if (!MainTrackerIndexPtr)
			{
				MainProxy.RegisterSlotNodeWithAnimInstance(LinkedSlotTrackerPair.Key);
				MainTrackerIndexPtr = MainProxy.SlotNameToTrackerIndex.Find(LinkedSlotTrackerPair.Key);
			}

			// Update slot information for main instance.
			{
				const FMontageActiveSlotTracker & LinkedTracker = SlotWeightTracker[GetBufferReadIndex()][LinkedSlotTrackerPair.Value];
				FMontageActiveSlotTracker& MainTracker = MainProxy.SlotWeightTracker[MainProxy.GetBufferWriteIndex()][*MainTrackerIndexPtr];
				
				MainTracker.MontageLocalWeight = FMath::Max(MainTracker.MontageLocalWeight, LinkedTracker.MontageLocalWeight);
				MainTracker.NodeGlobalWeight = FMath::Max(MainTracker.NodeGlobalWeight, LinkedTracker.NodeGlobalWeight);
				
				MainTracker.bIsRelevantThisTick = MainTracker.bIsRelevantThisTick || LinkedTracker.bIsRelevantThisTick;
				MainTracker.bWasRelevantOnPreviousTick = MainTracker.bWasRelevantOnPreviousTick || LinkedTracker.bWasRelevantOnPreviousTick;
			}
		}
	}
	
	InAnimInstance->NotifyQueue.Append(NotifyQueue);
	InAnimInstance->NotifyQueue.ApplyMontageNotifies(*this);

#if ENABLE_ANIM_DRAW_DEBUG
	FlushQueuedDebugDrawItems(InAnimInstance->GetSkelMeshComponent()->GetOwner(), InAnimInstance->GetSkelMeshComponent()->GetWorld());
#endif

#if ENABLE_ANIM_LOGGING
	FMessageLog MessageLog(GetTargetLogNameForCurrentWorldType());
	const TArray<FLogMessageEntry>* Messages = LoggedMessagesMap.Find(NAME_Update);
	if (ensureMsgf(Messages, TEXT("PreUpdate isn't called. This could potentially cause other issues.")))
	{
		for (const FLogMessageEntry& Message : *Messages)
		{
			MessageLog.AddMessage(Message);
		}
	}
#endif
}

#if ENABLE_ANIM_DRAW_DEBUG
void FAnimInstanceProxy::FlushQueuedDebugDrawItems(AActor* InActor, UWorld* InWorld) const
{
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		for (const FQueuedDrawDebugItem& DebugItem : QueuedDrawDebugItems)
		{
			switch (DebugItem.ItemType)
			{
				case EDrawDebugItemType::OnScreenMessage: GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.f, DebugItem.Color, DebugItem.Message, false, DebugItem.TextScale); break;
				case EDrawDebugItemType::InWorldMessage: DrawDebugString(InWorld, DebugItem.StartLoc, DebugItem.Message, InActor, DebugItem.Color, DebugItem.LifeTime, false /*bDrawShadow*/, DebugItem.TextScale.X); break;
				case EDrawDebugItemType::DirectionalArrow: DrawDebugDirectionalArrow(InWorld, DebugItem.StartLoc, DebugItem.EndLoc.Value, DebugItem.Size, DebugItem.Color, DebugItem.bPersistentLines, DebugItem.LifeTime, DebugItem.DepthPriority, DebugItem.Thickness); break;
				case EDrawDebugItemType::Sphere: DrawDebugSphere(InWorld, DebugItem.Center, DebugItem.Radius, DebugItem.Segments, DebugItem.Color, DebugItem.bPersistentLines, DebugItem.LifeTime, DebugItem.DepthPriority, DebugItem.Thickness); break;
				case EDrawDebugItemType::Line: DrawDebugLine(InWorld, DebugItem.StartLoc, DebugItem.EndLoc.Value, DebugItem.Color, DebugItem.bPersistentLines, DebugItem.LifeTime, DebugItem.DepthPriority, DebugItem.Thickness); break;
				case EDrawDebugItemType::CoordinateSystem: DrawDebugCoordinateSystem(InWorld, DebugItem.StartLoc, DebugItem.Rotation, DebugItem.Size, DebugItem.bPersistentLines, DebugItem.LifeTime, DebugItem.DepthPriority, DebugItem.Thickness); break;
				case EDrawDebugItemType::Point: DrawDebugPoint(InWorld, DebugItem.StartLoc, DebugItem.Size, DebugItem.Color, DebugItem.bPersistentLines, DebugItem.LifeTime, DebugItem.DepthPriority); break;
				case EDrawDebugItemType::Circle:
				{
					const FMatrix RotationMatrix = FRotationMatrix::MakeFromZ(DebugItem.Direction.Value);
					const FVector ForwardVector = RotationMatrix.GetScaledAxis(EAxis::X);
					const FVector RightVector = RotationMatrix.GetScaledAxis(EAxis::Y);
					DrawDebugCircle(InWorld, DebugItem.Center, DebugItem.Radius, DebugItem.Segments, DebugItem.Color, DebugItem.bPersistentLines, DebugItem.LifeTime, DebugItem.DepthPriority, DebugItem.Thickness, ForwardVector, RightVector, false);
					break;
				}
				case EDrawDebugItemType::Cone: DrawDebugCone(InWorld, DebugItem.Center, DebugItem.Direction.Value, DebugItem.Length, DebugItem.AngleWidth, DebugItem.AngleHeight, DebugItem.Segments, DebugItem.Color, DebugItem.bPersistentLines, DebugItem.LifeTime, DebugItem.DepthPriority, DebugItem.Thickness); break;
				case EDrawDebugItemType::Capsule: DrawDebugCapsule(InWorld, DebugItem.Center, DebugItem.Size, DebugItem.Radius, DebugItem.Rotation.Quaternion(), DebugItem.Color, DebugItem.bPersistentLines, DebugItem.LifeTime, DebugItem.DepthPriority, DebugItem.Thickness); break;
			}
		}
	}

	QueuedDrawDebugItems.Reset();
}
#endif

void FAnimInstanceProxy::PostEvaluate(UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

#if WITH_EDITORONLY_DATA
	if (FAnimBlueprintDebugData* DebugData = GetAnimBlueprintDebugData())
	{
		DebugData->RecordNodeAttributeMaps(NodeInputAttributesThisFrame, NodeOutputAttributesThisFrame);
	}
#endif
	
	ClearObjects();

#if ENABLE_ANIM_DRAW_DEBUG
	FlushQueuedDebugDrawItems(InAnimInstance->GetSkelMeshComponent()->GetOwner(), InAnimInstance->GetSkelMeshComponent()->GetWorld());
#endif

#if ENABLE_ANIM_LOGGING
	FMessageLog MessageLog(GetTargetLogNameForCurrentWorldType());
	if(const TArray<FLogMessageEntry>* Messages = LoggedMessagesMap.Find(NAME_Evaluate))
	{
		for (const FLogMessageEntry& Message : *Messages)
		{
			MessageLog.AddMessage(Message);
		}
	}
#endif
}

void FAnimInstanceProxy::InitializeObjects(UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	SkeletalMeshComponent = InAnimInstance->GetSkelMeshComponent();
	if(UAnimInstance* MainAnimInstance = SkeletalMeshComponent->GetAnimInstance())
	{
		MainInstanceProxy = &MainAnimInstance->GetProxyOnAnyThread<FAnimInstanceProxy>();
		bUseMainInstanceMontageEvaluationData = InAnimInstance->IsUsingMainInstanceMontageEvaluationData();
	}

	if (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
	{
		Skeleton = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton();
	}
	else
	{
		Skeleton = nullptr;
	}

	// Calculate the number of skipped frames after this one due to URO and store it on our evaluation and update counters
	const FAnimUpdateRateParameters* RateParams = SkeletalMeshComponent->AnimUpdateRateParams;

	NumUroSkippedFrames_Update = 0;
	NumUroSkippedFrames_Eval = 0;
	if(RateParams)
	{
		const bool bDoUro = SkeletalMeshComponent->ShouldUseUpdateRateOptimizations();
		if(bDoUro)
		{
			NumUroSkippedFrames_Update = RateParams->UpdateRate - 1;

			const bool bDoEvalOptimization = RateParams->DoEvaluationRateOptimizations();
			if(bDoEvalOptimization)
			{
				NumUroSkippedFrames_Eval = RateParams->EvaluationRate - 1;
			}
		}
		else if(SkeletalMeshComponent->IsUsingExternalTickRateControl())
		{
			NumUroSkippedFrames_Update = NumUroSkippedFrames_Eval = SkeletalMeshComponent->GetExternalTickRate();
		}
	}

	UpdateCounter.SetMaxSkippedFrames(NumUroSkippedFrames_Update);
	EvaluationCounter.SetMaxSkippedFrames(NumUroSkippedFrames_Eval);
}

void FAnimInstanceProxy::ClearObjects()
{
	SkeletalMeshComponent = nullptr;
	MainInstanceProxy = nullptr;
	Skeleton = nullptr;
}

FAnimTickRecord& FAnimInstanceProxy::CreateUninitializedTickRecord(FAnimGroupInstance*& OutSyncGroupPtr, FName GroupName)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Sync.CreateUninitializedTickRecord(OutSyncGroupPtr, GroupName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FAnimTickRecord& FAnimInstanceProxy::CreateUninitializedTickRecordInScope(FAnimGroupInstance*& OutSyncGroupPtr, FName GroupName, EAnimSyncGroupScope Scope)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Sync.CreateUninitializedTickRecordInScope(*this, OutSyncGroupPtr, GroupName, Scope);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FAnimTickRecord& FAnimInstanceProxy::CreateUninitializedTickRecord(int32 GroupIndex, FAnimGroupInstance*& OutSyncGroupPtr)
{
	FName SyncGroupName = NAME_None;
	if(GroupIndex >= 0)
	{
		SyncGroupName = GetAnimClassInterface()->GetSyncGroupNames()[GroupIndex];
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CreateUninitializedTickRecord(OutSyncGroupPtr, SyncGroupName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FAnimTickRecord& FAnimInstanceProxy::CreateUninitializedTickRecordInScope(int32 GroupIndex, EAnimSyncGroupScope Scope, FAnimGroupInstance*& OutSyncGroupPtr)
{
	FName SyncGroupName = NAME_None;
	if(GroupIndex >= 0)
	{
		SyncGroupName = GetAnimClassInterface()->GetSyncGroupNames()[GroupIndex];
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CreateUninitializedTickRecordInScope(OutSyncGroupPtr, SyncGroupName, Scope);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FAnimInstanceProxy::MakeSequenceTickRecord(FAnimTickRecord& TickRecord, class UAnimSequenceBase* Sequence, bool bLooping, float PlayRate, float FinalBlendWeight, float& CurrentTime, FMarkerTickRecord& MarkerTickRecord) const
{
	TickRecord.SourceAsset = Sequence;
	TickRecord.TimeAccumulator = &CurrentTime;
	TickRecord.MarkerTickRecord = &MarkerTickRecord;
	TickRecord.PlayRateMultiplier = PlayRate;
	TickRecord.EffectiveBlendWeight = FinalBlendWeight;
	TickRecord.bLooping = bLooping;
	TickRecord.bIsEvaluator = false;
}

void FAnimInstanceProxy::MakeBlendSpaceTickRecord(
	FAnimTickRecord& TickRecord, class UBlendSpace* BlendSpace, const FVector& BlendInput, TArray<FBlendSampleData>& BlendSampleDataCache, FBlendFilter& BlendFilter, 
	bool bLooping, float PlayRate, float FinalBlendWeight, float& CurrentTime, FMarkerTickRecord& MarkerTickRecord) const
{
	TickRecord.SourceAsset = BlendSpace;
	TickRecord.BlendSpace.BlendSpacePositionX = BlendInput.X;
	TickRecord.BlendSpace.BlendSpacePositionY = BlendInput.Y;
	// This way of making a tick record is deprecated, so just set to defaults here rather than changing the API
	TickRecord.BlendSpace.bTeleportToTime = false; 
	TickRecord.BlendSpace.BlendSampleDataCache = &BlendSampleDataCache;
	TickRecord.BlendSpace.BlendFilter = &BlendFilter;
	TickRecord.TimeAccumulator = &CurrentTime;
	TickRecord.MarkerTickRecord = &MarkerTickRecord;
	TickRecord.PlayRateMultiplier = PlayRate;
	TickRecord.EffectiveBlendWeight = FinalBlendWeight;
	TickRecord.bLooping = bLooping;
	TickRecord.bIsEvaluator = false;
}

void FAnimInstanceProxy::MakePoseAssetTickRecord(FAnimTickRecord& TickRecord, class UPoseAsset* PoseAsset, float FinalBlendWeight) const
{
	TickRecord.SourceAsset = PoseAsset;
	TickRecord.EffectiveBlendWeight = FinalBlendWeight;
}

void FAnimInstanceProxy::SequenceAdvanceImmediate(UAnimSequenceBase* Sequence, bool bLooping, float PlayRate, float DeltaSeconds, float& CurrentTime, FMarkerTickRecord& MarkerTickRecord)
{
	FAnimTickRecord TickRecord;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MakeSequenceTickRecord(TickRecord, Sequence, bLooping, PlayRate, /*FinalBlendWeight=*/ 1.0f, CurrentTime, MarkerTickRecord);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FDeltaTimeRecord DeltaTimeRecord;
	TickRecord.DeltaTimeRecord = &DeltaTimeRecord;

	FAnimAssetTickContext TickContext(DeltaSeconds, RootMotionMode, true);
	TickRecord.SourceAsset->TickAssetPlayer(TickRecord, NotifyQueue, TickContext);
}

void FAnimInstanceProxy::BlendSpaceAdvanceImmediate(class UBlendSpace* BlendSpace, const FVector& BlendInput, TArray<FBlendSampleData>& BlendSampleDataCache, FBlendFilter& BlendFilter, bool bLooping, float PlayRate, float DeltaSeconds, float& CurrentTime, FMarkerTickRecord& MarkerTickRecord)
{
	FAnimTickRecord TickRecord;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MakeBlendSpaceTickRecord(TickRecord, BlendSpace, BlendInput, BlendSampleDataCache, BlendFilter, bLooping, PlayRate, /*FinalBlendWeight=*/ 1.0f, CurrentTime, MarkerTickRecord);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FDeltaTimeRecord DeltaTimeRecord;
	TickRecord.DeltaTimeRecord = &DeltaTimeRecord;

	FAnimAssetTickContext TickContext(DeltaSeconds, RootMotionMode, true);
	TickRecord.SourceAsset->TickAssetPlayer(TickRecord, NotifyQueue, TickContext);
}

void FAnimInstanceProxy::TickAssetPlayerInstances()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TickAssetPlayerInstances(CurrentDeltaSeconds);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FAnimInstanceProxy::TickAssetPlayerInstances(float InDeltaSeconds)
{
	Sync.TickAssetPlayerInstances(*this, InDeltaSeconds);
	Sync.TickSyncGroupWriteIndex();
}

void FAnimInstanceProxy::AddAnimNotifies(const TArray<FAnimNotifyEventReference>& NewNotifies, const float InstanceWeight)
{
	NotifyQueue.AddAnimNotifies(true, NewNotifies, InstanceWeight);
}

int32 FAnimInstanceProxy::GetSyncGroupIndexFromName(FName SyncGroupName) const
{
	if (AnimClassInterface)
	{
		return AnimClassInterface->GetSyncGroupIndex(SyncGroupName);
	}
	return INDEX_NONE;
}

bool FAnimInstanceProxy::GetTimeToClosestMarker(FName SyncGroup, FName MarkerName, float& OutMarkerTime) const
{
	return Sync.GetTimeToClosestMarker(SyncGroup, MarkerName, OutMarkerTime);
}

void FAnimInstanceProxy::AddAnimNotifyFromGeneratedClass(int32 NotifyIndex)
{
	if(NotifyIndex==INDEX_NONE)
	{
		return;
	}

	if (AnimClassInterface)
	{
		check(AnimClassInterface->GetAnimNotifies().IsValidIndex(NotifyIndex));
		const FAnimNotifyEvent* Notify = &AnimClassInterface->GetAnimNotifies()[NotifyIndex];
		NotifyQueue.AddAnimNotify(Notify, IAnimClassInterface::GetActualAnimClass(AnimClassInterface));
	}
}

bool FAnimInstanceProxy::HasMarkerBeenHitThisFrame(FName SyncGroup, FName MarkerName) const
{
	return Sync.HasMarkerBeenHitThisFrame(SyncGroup, MarkerName);
}

bool FAnimInstanceProxy::IsSyncGroupBetweenMarkers(FName InSyncGroupName, FName PreviousMarker, FName NextMarker, bool bRespectMarkerOrder) const
{
	return Sync.IsSyncGroupBetweenMarkers(InSyncGroupName, PreviousMarker, NextMarker, bRespectMarkerOrder);
}

FMarkerSyncAnimPosition FAnimInstanceProxy::GetSyncGroupPosition(FName InSyncGroupName) const
{
	return Sync.GetSyncGroupPosition(InSyncGroupName);
}

bool FAnimInstanceProxy::IsSyncGroupValid(FName InSyncGroupName) const
{
	return Sync.IsSyncGroupValid(InSyncGroupName);
}


void FAnimInstanceProxy::ReinitializeSlotNodes()
{
	SlotNameToTrackerIndex.Reset();
	SlotWeightTracker[0].Reset();
	SlotWeightTracker[1].Reset();
	
	// Increment counter
	SlotNodeInitializationCounter.Increment();
}

void FAnimInstanceProxy::RegisterSlotNodeWithAnimInstance(const FName& SlotNodeName)
{
	// verify if same slot node name exists
	// then warn users, this is invalid
	if (SlotNameToTrackerIndex.Contains(SlotNodeName))
	{
		if (GWarnOnDuplicateSlotNode)
		{
			UClass* ActualAnimClass = IAnimClassInterface::GetActualAnimClass(GetAnimClassInterface());
			FString ClassNameString = ActualAnimClass ? ActualAnimClass->GetName() : FString("Unavailable");
			if (IsInGameThread())
			{
				// message log access means we need to run this in the game thread
				FMessageLog("AnimBlueprintLog").Warning(FText::Format(LOCTEXT("AnimInstance_SlotNode", "SLOTNODE: '{0}' in animation instance class {1} already exists. Remove duplicates from the animation graph for this class."), FText::FromString(SlotNodeName.ToString()), FText::FromString(ClassNameString)));
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "SLOTNODE: '%ls' in animation instance class %ls already exists. Remove duplicates from the animation graph for this class.", *SlotNodeName.ToString(), *ClassNameString);
			}
		}
		return;
	}

	int32 SlotIndex = SlotWeightTracker[0].Num();

	SlotNameToTrackerIndex.Add(SlotNodeName, SlotIndex);
	SlotWeightTracker[0].Add(FMontageActiveSlotTracker());
	SlotWeightTracker[1].Add(FMontageActiveSlotTracker());
}

void FAnimInstanceProxy::UpdateSlotNodeWeight(const FName& SlotNodeName, float InMontageLocalWeight, float InNodeGlobalWeight)
{
	const int32* TrackerIndexPtr = SlotNameToTrackerIndex.Find(SlotNodeName);
	if (TrackerIndexPtr)
	{
		FMontageActiveSlotTracker& Tracker = SlotWeightTracker[GetBufferWriteIndex()][*TrackerIndexPtr];
		Tracker.MontageLocalWeight = InMontageLocalWeight;
		Tracker.NodeGlobalWeight = InNodeGlobalWeight;

		// Count as relevant if we are weighted in
		Tracker.bIsRelevantThisTick = Tracker.bIsRelevantThisTick || FAnimWeight::IsRelevant(InMontageLocalWeight);
	}
}

bool FAnimInstanceProxy::GetSlotInertializationRequest(const FName& SlotName, UE::Anim::FSlotInertializationRequest& OutRequest)
{
	return false;
}

bool FAnimInstanceProxy::GetSlotInertializationRequestData(const FName& SlotName, FInertializationRequest& OutRequest)
{
	const FName GroupName = Skeleton ? Skeleton->GetSlotGroupName(SlotName) : NAME_None;
	if (const FInertializationRequest* FoundRequest = GetSlotGroupInertializationRequestDataMap().Find(GroupName))
	{
		OutRequest = *FoundRequest;
		return true;
	}

	return false;
}

void FAnimInstanceProxy::ClearSlotNodeWeights()
{
	TArray<FMontageActiveSlotTracker>& SlotWeightTracker_Read = SlotWeightTracker[GetBufferReadIndex()];
	TArray<FMontageActiveSlotTracker>& SlotWeightTracker_Write = SlotWeightTracker[GetBufferWriteIndex()];

	for (int32 TrackerIndex = 0; TrackerIndex < SlotWeightTracker_Write.Num(); TrackerIndex++)
	{
		SlotWeightTracker_Write[TrackerIndex] = FMontageActiveSlotTracker();
		SlotWeightTracker_Write[TrackerIndex].bWasRelevantOnPreviousTick = SlotWeightTracker_Read[TrackerIndex].bIsRelevantThisTick;
	}
}

bool FAnimInstanceProxy::IsSlotNodeRelevantForNotifies(const FName& SlotNodeName) const
{
	const int32* TrackerIndexPtr = SlotNameToTrackerIndex.Find(SlotNodeName);
	if (TrackerIndexPtr)
	{
		const FMontageActiveSlotTracker& Tracker = SlotWeightTracker[GetBufferReadIndex()][*TrackerIndexPtr];
		return (Tracker.bIsRelevantThisTick || Tracker.bWasRelevantOnPreviousTick);
	}

	return false;
}

float FAnimInstanceProxy::GetSlotNodeGlobalWeight(const FName& SlotNodeName) const
{
	const int32* TrackerIndexPtr = SlotNameToTrackerIndex.Find(SlotNodeName);
	if (TrackerIndexPtr)
	{
		const FMontageActiveSlotTracker& Tracker = SlotWeightTracker[GetBufferReadIndex()][*TrackerIndexPtr];
		return Tracker.NodeGlobalWeight;
	}

	return 0.f;
}

float FAnimInstanceProxy::GetSlotMontageGlobalWeight(const FName& SlotNodeName) const
{
	const int32* TrackerIndexPtr = SlotNameToTrackerIndex.Find(SlotNodeName);
	if (TrackerIndexPtr)
	{
		const FMontageActiveSlotTracker& Tracker = SlotWeightTracker[GetBufferReadIndex()][*TrackerIndexPtr];
		return Tracker.MontageLocalWeight * Tracker.NodeGlobalWeight;
	}

	return 0.f;
}

float FAnimInstanceProxy::GetSlotMontageLocalWeight(const FName& SlotNodeName) const
{
	const int32* TrackerIndexPtr = SlotNameToTrackerIndex.Find(SlotNodeName);
	if (TrackerIndexPtr)
	{
		const FMontageActiveSlotTracker& Tracker = SlotWeightTracker[GetBufferReadIndex()][*TrackerIndexPtr];
		return Tracker.MontageLocalWeight;
	}

	return 0.f;
}

float FAnimInstanceProxy::CalcSlotMontageLocalWeight(const FName& SlotNodeName) const
{
	float out_SlotNodeLocalWeight, out_SourceWeight, out_TotalNodeWeight;
	GetSlotWeight(SlotNodeName, out_SlotNodeLocalWeight, out_SourceWeight, out_TotalNodeWeight);

	return out_SlotNodeLocalWeight;
}

const FAnimNode_Base* FAnimInstanceProxy::GetCheckedNodeFromIndexUntyped(int32 NodeIdx, UScriptStruct* RequiredStructType) const
{
	const FAnimNode_Base* NodePtr = nullptr;
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		const int32 InstanceIdx = AnimNodeProperties.Num() - 1 - NodeIdx;

		if (AnimNodeProperties.IsValidIndex(InstanceIdx))
		{
			FStructProperty* NodeProperty = AnimNodeProperties[InstanceIdx];

			if (NodeProperty->Struct->IsChildOf(RequiredStructType))
			{
				NodePtr = NodeProperty->ContainerPtrToValuePtr<FAnimNode_Base>(AnimInstanceObject);
			}
			else
			{
				checkfSlow(false, TEXT("Requested a node of type %s but found node of type %s"), *RequiredStructType->GetName(), *NodeProperty->Struct->GetName());
			}
		}
		else
		{
			checkfSlow(false, TEXT("Requested node of type %s at index %d/%d, index out of bounds."), *RequiredStructType->GetName(), NodeIdx, InstanceIdx);
		}
	}

	checkfSlow(NodePtr, TEXT("Requested node at index %d not found!"), NodeIdx);

	return NodePtr;
}

FAnimNode_Base* FAnimInstanceProxy::GetCheckedMutableNodeFromIndexUntyped(int32 NodeIdx, UScriptStruct* RequiredStructType)
{
	return const_cast<FAnimNode_Base*>(GetCheckedNodeFromIndexUntyped(NodeIdx, RequiredStructType));
}

const FAnimNode_Base* FAnimInstanceProxy::GetNodeFromIndexUntyped(int32 NodeIdx, UScriptStruct* RequiredStructType) const
{
	const FAnimNode_Base* NodePtr = nullptr;
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		const int32 InstanceIdx = AnimNodeProperties.Num() - 1 - NodeIdx;

		if (AnimNodeProperties.IsValidIndex(InstanceIdx))
		{
			FStructProperty* NodeProperty = AnimNodeProperties[InstanceIdx];

			if (NodeProperty->Struct->IsChildOf(RequiredStructType))
			{
				NodePtr = NodeProperty->ContainerPtrToValuePtr<FAnimNode_Base>(AnimInstanceObject);
			}
		}
	}

	return NodePtr;
}

FAnimNode_Base* FAnimInstanceProxy::GetMutableNodeFromIndexUntyped(int32 NodeIdx, UScriptStruct* RequiredStructType)
{
	return const_cast<FAnimNode_Base*>(GetNodeFromIndexUntyped(NodeIdx, RequiredStructType));
}

void FAnimInstanceProxy::RecalcRequiredBones(USkeletalMeshComponent* Component, UObject* Asset)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// Use the shared bone container
	RequiredBones = Component->GetSharedRequiredBones();

	// The first anim instance will initialize the required bones, all others will re-use it
	if (!RequiredBones->IsValid())
	{
		RequiredBones->InitializeTo(Component->RequiredBones, Component->GetCurveFilterSettings(), *Asset);

		// If there is a ref pose override, we want to replace ref pose in RequiredBones
		// Update ref pose in required bones structure (either set it, or clear it, depending on if one is set on the Component)
		RequiredBones->SetRefPoseOverride(Component->GetRefPoseOverride());
	}

	// If this instance can accept input poses, initialise the input pose container
	if (DefaultLinkedInstanceInputNode)
	{
		DefaultLinkedInstanceInputNode->CachedInputPose.SetBoneContainer(RequiredBones.Get());
		
		// SetBoneContainer allocates space for bone data but leaves it uninitalized.
		DefaultLinkedInstanceInputNode->bIsCachedInputPoseInitialized = false;
	}

	// When RequiredBones mapping has changed, AnimNodes need to update their bones caches.
	bBoneCachesInvalidated = true;
}

void FAnimInstanceProxy::RecalcRequiredCurves(const UE::Anim::FCurveFilterSettings& CurveFilterSettings)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	RequiredBones->CacheRequiredAnimCurves(CurveFilterSettings);
	bBoneCachesInvalidated = true;
}

void FAnimInstanceProxy::RecalcRequiredCurves(const FCurveEvaluationOption& CurveEvalOption)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RequiredBones->CacheRequiredAnimCurveUids(CurveEvalOption);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	bBoneCachesInvalidated = true;
}

void FAnimInstanceProxy::UpdateAnimation()
{
	LLM_SCOPE(ELLMTag::Animation);
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FMemMark Mark(FMemStack::Get());
	FAnimationUpdateSharedContext SharedContext;
	FAnimationUpdateContext Context(this, CurrentDeltaSeconds, &SharedContext);
	UE_LOG_CONTEXT("AnimBlueprint", FTopLevelAssetPath(GetAnimInstanceObject() ? GetAnimInstanceObject()->GetClass() : nullptr));

	if(AnimClassInterface && AnimClassInterface->GetAnimBlueprintFunctions().Num() > 0)
	{
		Context.SetNodeId(AnimClassInterface->GetAnimBlueprintFunctions()[0].OutputPoseNodeIndex);
	}

	UpdateAnimation_WithRoot(Context, RootNode, NAME_AnimGraph);
	
	// Tick syncing
	Sync.TickAssetPlayerInstances(*this, CurrentDeltaSeconds);
	
	// Now that we have updated our asset players we record their tick records from read buffer (meaning they have up-to-date info).
#if ANIM_TRACE_ENABLED
	{
		for (const TTuple<FName, FAnimGroupInstance>& TickRecordGroup : Sync.GetSyncGroupMapRead())
		{
			for (const FAnimTickRecord& TickRecord : TickRecordGroup.Value.ActivePlayers)
			{
				TRACE_ANIM_TICK_RECORD(Context, TickRecord)
			}
		}

		for (const FAnimTickRecord& TickRecord : Sync.GetUngroupedActivePlayersRead())
		{
			TRACE_ANIM_TICK_RECORD(Context, TickRecord)
		}
	}
#endif
}

void FAnimInstanceProxy::UpdateAnimation_WithRoot(const FAnimationUpdateContext& InContext, FAnimNode_Base* InRootNode, FName InLayerName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	ANIM_MT_SCOPE_CYCLE_COUNTER(ProxyUpdateAnimation, !IsInGameThread());
	FScopeCycleCounterUObject AnimScope(bUpdatingRoot ? nullptr : GetAnimInstanceObject());


#if DO_CHECK
	ensureMsgf(CastChecked<UAnimInstance>(GetAnimInstanceObject())->IsUpdateAnimationEnabled(), TEXT("Updating AnimInstance with UpdateAnimation disabled: %s"), *CastChecked<UAnimInstance>(GetAnimInstanceObject())->GetName());
#endif
	if (!CastChecked<UAnimInstance>(GetAnimInstanceObject())->IsUpdateAnimationEnabled())
	{
		return;
	}

	if(InRootNode == RootNode)
	{
		if(bInitializeSubsystems && AnimClassInterface)
		{
			AnimClassInterface->ForEachSubsystem(GetAnimInstanceObject(), [this](const FAnimSubsystemInstanceContext& InContext)
			{
				InContext.SubsystemInstance.Initialize_WorkerThread();
				return EAnimSubsystemEnumeration::Continue;
			});
			bInitializeSubsystems = false;
		}
		
		if(bDeferRootNodeInitialization)
		{
			InitializeRootNode_WithRoot(RootNode);

			if(AnimClassInterface)
			{
				// Initialize linked sub graphs
				for(const FStructProperty* LayerNodeProperty : AnimClassInterface->GetLinkedAnimLayerNodeProperties())
				{
					if(FAnimNode_LinkedAnimLayer* LayerNode = LayerNodeProperty->ContainerPtrToValuePtr<FAnimNode_LinkedAnimLayer>(AnimInstanceObject))
					{
						if(UAnimInstance* LinkedInstance = LayerNode->GetTargetInstance<UAnimInstance>())
						{
							FAnimationInitializeContext InitContext(this);
							LayerNode->InitializeSubGraph_AnyThread(InitContext);
							FAnimationCacheBonesContext CacheBonesContext(this);
							LayerNode->CacheBonesSubGraph_AnyThread(CacheBonesContext);
						}
					}
				}
			}

			bDeferRootNodeInitialization = false;
		}

		// Call the correct override point if this is the root node
		CacheBones();
	}
	else
	{
		CacheBones_WithRoot(InRootNode);
	}

	// update native update
	if(!bUpdatingRoot)
	{
		// Make sure we only update this once the first time we update, as we can re-call this function
		// from other linked instances with grouped layers
		if(FrameCounterForUpdate != GFrameCounter)
		{
			if(AnimClassInterface)
			{
				AnimClassInterface->ForEachSubsystem(GetAnimInstanceObject(), [this](const FAnimSubsystemInstanceContext& InContext)
				{
					FAnimSubsystemParallelUpdateContext Context(InContext, *this, CurrentDeltaSeconds);
					InContext.Subsystem.OnPreUpdate_WorkerThread(Context);
					return EAnimSubsystemEnumeration::Continue;
				});
			}

			{
				SCOPE_CYCLE_COUNTER(STAT_NativeThreadSafeUpdateAnimation);
				CastChecked<UAnimInstance>(GetAnimInstanceObject())->NativeThreadSafeUpdateAnimation(CurrentDeltaSeconds);
			}

			{
				SCOPE_CYCLE_COUNTER(STAT_BlueprintUpdateAnimation);
				CastChecked<UAnimInstance>(GetAnimInstanceObject())->BlueprintThreadSafeUpdateAnimation(CurrentDeltaSeconds);
			}

			{
				SCOPE_CYCLE_COUNTER(STAT_NativeUpdateAnimation);
				Update(CurrentDeltaSeconds);
			}

			if(AnimClassInterface)
			{
				AnimClassInterface->ForEachSubsystem(GetAnimInstanceObject(), [this](const FAnimSubsystemInstanceContext& InContext)
				{
					FAnimSubsystemParallelUpdateContext Context(InContext, *this, CurrentDeltaSeconds);
					InContext.Subsystem.OnPostUpdate_WorkerThread(Context);
					return EAnimSubsystemEnumeration::Continue;
				});
			}
			
			FrameCounterForUpdate = GFrameCounter;
		}
	}

	// Update root
	{
		// We re-enter this function when we call layer graphs linked to the main graph. In these cases we
		// dont want to perform duplicate work
		TGuardValue<bool> ScopeGuard(bUpdatingRoot, true);

		// Anything syncing within this scope is subject to sync groups.
		// We only enable syncing here for the main instance or post process instance
		// We also fall back to enabling this sync scope if there is not one already enabled (there must always be one)
		const bool bEnableSyncScope =	GetAnimInstanceObject() == GetSkelMeshComponent()->GetAnimInstance() ||
										GetAnimInstanceObject() == GetSkelMeshComponent()->GetPostProcessInstance() || 
										InContext.GetMessage<UE::Anim::FAnimSyncGroupScope>() == nullptr;
		UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimSyncGroupScope> Message(bEnableSyncScope, InContext, InContext);

		// update all nodes
		if(InRootNode == RootNode)
		{
			// Call the correct override point if this is the root node
			UpdateAnimationNode(InContext);
		}
		else
		{
			UpdateAnimationNode_WithRoot(InContext, InRootNode, InLayerName);
		}
	}
}

void FAnimInstanceProxy::PreEvaluateAnimation(UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	InitializeObjects(InAnimInstance);

	// Re-cache Component Transforms if needed
	// When playing root motion is the CharacterMovementComp who triggers the animation update and this happens before root motion is consumed and the position of the character is updated for this frame
	// which means that at the point ComponentTransform is cached in the PreUpdate function it contain the previous frame transform
	if (SkeletalMeshComponent && SkeletalMeshComponent->IsPlayingRootMotion())
	{
		ComponentTransform = SkeletalMeshComponent->GetComponentTransform();
		ComponentRelativeTransform = SkeletalMeshComponent->GetRelativeTransform();
		ActorTransform = SkeletalMeshComponent->GetOwner() ? SkeletalMeshComponent->GetOwner()->GetActorTransform() : FTransform::Identity;
	}

#if ENABLE_ANIM_LOGGING
	LoggedMessagesMap.FindOrAdd(NAME_Evaluate).Reset();
#endif
}

void FAnimInstanceProxy::EvaluateAnimation(FPoseContext& Output)
{
	EvaluateAnimation_WithRoot(Output, RootNode);
}

void FAnimInstanceProxy::EvaluateAnimation_WithRoot(FPoseContext& Output, FAnimNode_Base* InRootNode)
{
	TRACE_SCOPED_ANIM_GRAPH(Output);
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	ANIM_MT_SCOPE_CYCLE_COUNTER(EvaluateAnimInstance, !IsInGameThread());

	if(InRootNode == RootNode)
	{
		// Call the correct override point if this is the root node
		CacheBones();
	}
	else
	{
		CacheBones_WithRoot(InRootNode);
	}

	// Evaluate native code if implemented, otherwise evaluate the node graph
	if (!Evaluate_WithRoot(Output, InRootNode))
	{
		EvaluateAnimationNode_WithRoot(Output, InRootNode);
	}
}

void FAnimInstanceProxy::CacheBones()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/Graph"));

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// If bone caches have been invalidated, have AnimNodes refresh those.
	if (bBoneCachesInvalidated && RootNode)
	{
		CacheBonesRecursionCounter++;

		CachedBonesCounter.Increment();
		FAnimationCacheBonesContext Context(this);

		TRACE_SCOPED_ANIM_GRAPH(Context);

		RootNode->CacheBones_AnyThread(Context);

		CacheBonesRecursionCounter--;

		check(CacheBonesRecursionCounter >= 0);

		if(CacheBonesRecursionCounter == 0)
		{
			bBoneCachesInvalidated = false;
		}
	}
}

void FAnimInstanceProxy::CacheBones_WithRoot(FAnimNode_Base* InRootNode)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/Graph"));

	// If bone caches have been invalidated, have AnimNodes refresh those.
	if (bBoneCachesInvalidated && InRootNode)
	{
		CacheBonesRecursionCounter++;

		if(InRootNode == RootNode)
		{
			CachedBonesCounter.Increment();
		}
		FAnimationCacheBonesContext Context(this);
		InRootNode->CacheBones_AnyThread(Context);

		CacheBonesRecursionCounter--;

		check(CacheBonesRecursionCounter >= 0);

		if(CacheBonesRecursionCounter == 0)
		{
			bBoneCachesInvalidated = false;
		}
	}
}

void FAnimInstanceProxy::EvaluateAnimationNode(FPoseContext& Output)
{
	EvaluateAnimationNode_WithRoot(Output, RootNode);
}

void FAnimInstanceProxy::EvaluateAnimationNode_WithRoot(FPoseContext& Output, FAnimNode_Base* InRootNode)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/Graph"));

	if (InRootNode != nullptr)
	{
		ANIM_MT_SCOPE_CYCLE_COUNTER(EvaluateAnimGraph, !IsInGameThread());
		if(InRootNode == RootNode)
		{
			EvaluationCounter.Increment();

			if(AnimClassInterface && AnimClassInterface->GetAnimBlueprintFunctions().Num() > 0)
			{
				Output.SetNodeId(INDEX_NONE);
				Output.SetNodeId(AnimClassInterface->GetAnimBlueprintFunctions()[0].OutputPoseNodeIndex);
			}
		}

		TRACE_SCOPED_ANIM_NODE(Output);
		
		InRootNode->Evaluate_AnyThread(Output);
	}
	else
	{
		Output.ResetToRefPose();
	}
}

// for now disable because it will not work with single node instance
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define DEBUG_MONTAGEINSTANCE_WEIGHT 0
#else
#define DEBUG_MONTAGEINSTANCE_WEIGHT 1
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

void FAnimInstanceProxy::SlotEvaluatePose(const FName& SlotNodeName, const FCompactPose& SourcePose, const FBlendedCurve& SourceCurve, float InSourceWeight, FCompactPose& BlendedPose, FBlendedCurve& BlendedCurve, float InBlendWeight, float InTotalNodeWeight)
{
	UE::Anim::FStackAttributeContainer TempAttributes;

	const FAnimationPoseData SourceAnimationPoseData(*const_cast<FCompactPose*>(&SourcePose), *const_cast<FBlendedCurve*>(&SourceCurve), TempAttributes);
	FAnimationPoseData BlendedAnimationPoseData(BlendedPose, BlendedCurve, TempAttributes);

	SlotEvaluatePose(SlotNodeName, SourceAnimationPoseData, InSourceWeight, BlendedAnimationPoseData, InBlendWeight, InTotalNodeWeight);
}

void FAnimInstanceProxy::SlotEvaluatePoseWithBlendProfiles(const FName& SlotNodeName, const FAnimationPoseData& SourceAnimationPoseData, float InSourceWeight, FAnimationPoseData& OutBlendedAnimationPoseData, float InBlendWeight)
{
	const FCompactPose& SourcePose = SourceAnimationPoseData.GetPose();
	const FBlendedCurve& SourceCurve = SourceAnimationPoseData.GetCurve();
	const UE::Anim::FStackAttributeContainer& SourceAttributes = SourceAnimationPoseData.GetAttributes();

	FCompactPose& BlendedPose = OutBlendedAnimationPoseData.GetPose();
	FBlendedCurve& BlendedCurve = OutBlendedAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& BlendedAttributes = OutBlendedAnimationPoseData.GetAttributes();

	if (InBlendWeight <= ZERO_ANIMWEIGHT_THRESH)
	{
		BlendedPose = SourcePose;
		BlendedCurve = SourceCurve;
		BlendedAttributes = SourceAttributes;
		return;
	}

	// Get the array of per bone weight totals.
	FBlendProfileScratchData& BlendProfileScratchData = FBlendProfileScratchData::Get();
	TArray<TArray<float>>& PerBoneWeights = BlendProfileScratchData.PerBoneWeights;
	TArray<float>& PerBoneWeightTotals = BlendProfileScratchData.PerBoneWeightTotals;
	TArray<float>& BoneBlendProfileScales = BlendProfileScratchData.BoneBlendProfileScales;
	check(BlendProfileScratchData.IsEmpty());

	const int32 NumBones = RequiredBones->GetCompactPoseNumBones();
	PerBoneWeightTotals.AddUninitialized(NumBones);
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		PerBoneWeightTotals[BoneIndex] = 0.0f;
	}
	PerBoneWeights.Reset(GetMontageEvaluationData().Num());

	//------------------------------------------
	// Figure out what poses we need to blend.
	//------------------------------------------
	// Make a list of all the montages that use the slot we're interested in.
	// Split this in an additive and non additive list.
	check(GetMontageEvaluationData().Num() < 255); // Make sure we're in limits and not blending more than 255 poses (that would indicate another issue anyway)
	TArray<FSlotEvaluationPose>& Poses = BlendProfileScratchData.Poses;
	TArray<FSlotEvaluationPose>& AdditivePoses = BlendProfileScratchData.AdditivePoses;
	TArray<uint8, TInlineAllocator<8>>& PoseIndices = BlendProfileScratchData.PoseIndices;
	TArray<uint8, TInlineAllocator<8>>& AdditivePoseIndices = BlendProfileScratchData.AdditivePoseIndices;
	TArray<float, TInlineAllocator<8>>& BlendingWeights = BlendProfileScratchData.BlendingWeights;
	TArray<const FCompactPose*, TInlineAllocator<8>>& BlendingPoses = BlendProfileScratchData.BlendingPoses;
	TArray<const FBlendedCurve*, TInlineAllocator<8>>& BlendingCurves = BlendProfileScratchData.BlendingCurves;
	TArray<const UE::Anim::FStackAttributeContainer*, TInlineAllocator<8>>& BlendingAttributes = BlendProfileScratchData.BlendingAttributes;

	// Gather all the poses we're interested in.
	int32 CurrentPoseIndex = 0;
	float NonAdditiveTotalWeight = 0.0f;
	float AdditiveTotalWeight = 0.0f;
	for (int32 Index = 0; Index < GetMontageEvaluationData().Num(); ++Index)
	{
		const FMontageEvaluationState& EvalState = GetMontageEvaluationData()[Index];
		const UAnimMontage* Montage = EvalState.Montage.Get();
		if (Montage && Montage->IsValidSlot(SlotNodeName))
		{
			const UBlendProfile* BlendProfile = EvalState.ActiveBlendProfile;
			const FAnimTrack* AnimTrack = Montage->GetAnimationData(SlotNodeName);
			const EAdditiveAnimationType AdditiveAnimType = AnimTrack->IsAdditive()
				? (AnimTrack->IsRotationOffsetAdditive() ? AAT_RotationOffsetMeshSpace : AAT_LocalSpaceBase)
				: AAT_None;

			// Bone array has to be allocated prior to calling GetPoseFromAnimTrack.
			FSlotEvaluationPose NewPose(EvalState.BlendInfo.GetBlendedValue(), AdditiveAnimType);
			NewPose.Pose.SetBoneContainer(RequiredBones.Get());
			NewPose.Curve.InitFrom(*RequiredBones);

			// Extract pose from Track.
			FAnimExtractContext ExtractionContext(static_cast<double>(EvalState.MontagePosition), Montage->HasRootMotion() && RootMotionMode != ERootMotionMode::NoRootMotionExtraction, EvalState.DeltaTimeRecord);
			ExtractionContext.InterpolationOverride = InterpolationOverride;
			FAnimationPoseData NewAnimationPoseData(NewPose);
			AnimTrack->GetAnimationPose(NewAnimationPoseData, ExtractionContext);

			// Add montage curves.
			FBlendedCurve MontageCurve;
			MontageCurve.InitFrom(*RequiredBones);
			const FAnimExtractContext Context(static_cast<double>(EvalState.MontagePosition));
			Montage->EvaluateCurveData(MontageCurve, Context);
			NewPose.Curve.Combine(MontageCurve);

			// Capture non-additive poses only in the non-additive pass.
			if (AdditiveAnimType == AAT_None)
			{
				Poses.Add(FSlotEvaluationPose(MoveTemp(NewPose)));
				PoseIndices.Add(static_cast<uint8>(CurrentPoseIndex));
				NonAdditiveTotalWeight += NewPose.Weight;
			}
			else // Grab additives in the additive pass.
			{
				AdditivePoses.Add(FSlotEvaluationPose(MoveTemp(NewPose)));
				AdditivePoseIndices.Add(static_cast<uint8>(CurrentPoseIndex));
				AdditiveTotalWeight += NewPose.Weight;
			}

			PerBoneWeights.AddDefaulted();
			PerBoneWeights[CurrentPoseIndex].AddUninitialized(NumBones);

			const float PoseWeight = EvalState.BlendInfo.GetBlendedValue();
			if (BlendProfile)
			{
				BlendProfile->FillBoneScalesArray(BoneBlendProfileScales, *RequiredBones);
				for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
				{
					PerBoneWeights[CurrentPoseIndex][BoneIndex] = BlendProfile->CalculateBoneWeight(BoneBlendProfileScales[BoneIndex], BlendProfile->Mode, EvalState.BlendInfo, EvalState.BlendStartAlpha, NewPose.Weight, false);
				}
			}
			else //  This pose doesn't use a blend profile, use the pose weight.
			{
				for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
				{
					PerBoneWeights[CurrentPoseIndex][BoneIndex] = PoseWeight;
				}
			}

			// Sum up the total weights per bone for non-additive poses.
			if (AdditiveAnimType == AAT_None)
			{
				for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
				{
					PerBoneWeightTotals[BoneIndex] += PerBoneWeights[CurrentPoseIndex][BoneIndex];
				}
			}
			
			CurrentPoseIndex++;
		} // If montage slot is valid.
	} // For all montage eval data.

	// Register the source pose if we have any, but don't include it in our normalizations.
	const float SourceWeight = FMath::Clamp(InSourceWeight, 0.0f, 1.0f);
	const bool bHasSourcePose = (SourceWeight > ZERO_ANIMWEIGHT_THRESH);
	if (bHasSourcePose)
	{
		const int32 SourcePoseIndex = PerBoneWeights.AddDefaulted();
		PerBoneWeights[SourcePoseIndex].Reset(NumBones);
		PerBoneWeights[SourcePoseIndex].AddUninitialized(NumBones);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			PerBoneWeights[SourcePoseIndex][BoneIndex] = SourceWeight;
		}

		PoseIndices.Add(SourcePoseIndex);
		CurrentPoseIndex++;
	}
	
	// Normalize non additive weights.
	const float NormalizeThreshold = bHasSourcePose ? (1.0f + ZERO_ANIMWEIGHT_THRESH) : ZERO_ANIMWEIGHT_THRESH;
	const int32 NumPosesToNormalize = Poses.Num();
	for (int32 PoseIndex = 0; PoseIndex < NumPosesToNormalize; ++PoseIndex)
	{
		const int32 PoseWeightIndex = PoseIndices[PoseIndex];
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const float TotalWeight = PerBoneWeightTotals[BoneIndex];
			if (TotalWeight > NormalizeThreshold)
			{
				PerBoneWeights[PoseWeightIndex][BoneIndex] /= TotalWeight;
			}
			else
			{
				if (!bHasSourcePose)
				{
					PerBoneWeights[PoseWeightIndex][BoneIndex] = 1.0f;
				}
			}
		}
	}

	// Calculate the source pose weights.
	if (bHasSourcePose)
	{
		const int32 PoseWeightIndex = PoseIndices.Num() - 1;
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			float OthersWeightSum = 0.0f;
			for (int32 PoseIndex = 0; PoseIndex < PoseIndices.Num() - 1; ++PoseIndex)
			{
				OthersWeightSum += PerBoneWeights[PoseIndices[PoseIndex]][BoneIndex];
			}

			PerBoneWeights[PoseIndices[PoseWeightIndex]][BoneIndex] = 1.0f - OthersWeightSum;
		}
	}

	// Normalize non additive pose weights, as we need them for blending curves and attributes.
	if (NonAdditiveTotalWeight > 1.0f + ZERO_ANIMWEIGHT_THRESH)
	{
		for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
		{
			Poses[PoseIndex].Weight /= NonAdditiveTotalWeight;
		}
	}

	// NOTE: we don't normalize the additive poses.

	//----------------------------------
	// Build the blend arrays.
	//----------------------------------
	const int32 NumPoses = Poses.Num() + (bHasSourcePose ? 1 : 0); // Include the source input pose when doing non-additive blends.
	BlendingPoses.AddUninitialized(NumPoses);
	BlendingCurves.AddUninitialized(NumPoses);
	BlendingAttributes.AddUninitialized(NumPoses);
	BlendingWeights.AddUninitialized(NumPoses);

	for (int32 Index = 0; Index < Poses.Num(); Index++)
	{
		BlendingPoses[Index] = &Poses[Index].Pose;
		BlendingCurves[Index] = &Poses[Index].Curve;
		BlendingAttributes[Index] = &Poses[Index].Attributes;
		BlendingWeights[Index] = Poses[Index].Weight;
	}

	// Add the source pose to the blends.
	if (bHasSourcePose)
	{
		const int32 SourceIndex = NumPoses - 1;
		BlendingPoses[SourceIndex] = &SourcePose;
		BlendingCurves[SourceIndex] = &SourceCurve;
		BlendingAttributes[SourceIndex] = &SourceAttributes;
		BlendingWeights[SourceIndex] = (NonAdditiveTotalWeight > 1.0f + ZERO_ANIMWEIGHT_THRESH) ? SourceWeight / NonAdditiveTotalWeight : SourceWeight; // Normalize the source weight if needed.
	}

	//------------------------------------------
	// Blend the transforms.
	//------------------------------------------
	if (Poses.Num() == 0) // There are only additive poses.
	{
		check(AdditivePoseIndices.Num() > 0); // If there are no non-additive poses, there should be at least an additive.
		BlendedPose = SourcePose;
		BlendedCurve = SourceCurve;
		BlendedAttributes = SourceAttributes;
	}
	else
	{
		// Perform the actual bone transform blending.
		for (int32 PoseIndex = 0; PoseIndex < NumPoses; ++PoseIndex)
		{
			const int32 WeightPoseIndex = PoseIndices[PoseIndex];
			if (PoseIndex == 0)
			{
				for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
				{
					const FCompactPoseBoneIndex CompactBoneIndex(BoneIndex);
					BlendTransform<ETransformBlendMode::Overwrite>(
						(*BlendingPoses[PoseIndex])[CompactBoneIndex],
						BlendedPose[CompactBoneIndex],
						PerBoneWeights[WeightPoseIndex][BoneIndex]);
				}
			}
			else
			{
				for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
				{
					const FCompactPoseBoneIndex CompactBoneIndex(BoneIndex);
					BlendTransform<ETransformBlendMode::Accumulate>(
						(*BlendingPoses[PoseIndex])[CompactBoneIndex],
						BlendedPose[CompactBoneIndex],
						PerBoneWeights[WeightPoseIndex][BoneIndex]);
				}
			}
		}

		BlendedPose.NormalizeRotations();
	}
	
	// Additive blends.
	for (int32 PoseIndex = 0; PoseIndex < AdditivePoses.Num(); ++PoseIndex)
	{
		FCompactPose& AdditivePose = AdditivePoses[PoseIndex].Pose;
		const int32 WeightPoseIndex = AdditivePoseIndices[PoseIndex];
		if (AdditivePoses[PoseIndex].AdditiveType == AAT_RotationOffsetMeshSpace)
		{
			FAnimationRuntime::ConvertPoseToMeshRotation(BlendedPose);

			// Apply additive.
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				const FCompactPoseBoneIndex CompactBoneIndex(BoneIndex);
				FTransform Additive = AdditivePose[CompactBoneIndex];
				FTransform::BlendFromIdentityAndAccumulate(BlendedPose[CompactBoneIndex], Additive, ScalarRegister(PerBoneWeights[WeightPoseIndex][BoneIndex]));
			}

			FAnimationRuntime::ConvertMeshRotationPoseToLocalSpace(BlendedPose);
		}
		else
		{
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				const FCompactPoseBoneIndex CompactBoneIndex(BoneIndex);
				FTransform Additive = AdditivePose[CompactBoneIndex];
				FTransform::BlendFromIdentityAndAccumulate(BlendedPose[CompactBoneIndex], Additive, ScalarRegister(PerBoneWeights[WeightPoseIndex][BoneIndex]));
			}
		}

		BlendedPose.NormalizeRotations();
	} // For each additive pose.
	
	//------------------------------------------
	// Blend curves and attributes.
	//------------------------------------------
	// Non-additives.
	if (BlendingCurves.Num() > 0)
	{
		BlendCurves(BlendingCurves, BlendingWeights, OutBlendedAnimationPoseData.GetCurve());
	}

	if (BlendingAttributes.Num() > 0)
	{
		UE::Anim::Attributes::BlendAttributes(BlendingAttributes, BlendingWeights, OutBlendedAnimationPoseData.GetAttributes());
	}

	// Additives.
	for (int32 PoseIndex = 0; PoseIndex < AdditivePoses.Num(); ++PoseIndex)
	{
		FSlotEvaluationPose& AdditivePose = AdditivePoses[PoseIndex];
		const FAnimationPoseData AdditiveAnimationPoseData(AdditivePose);
		OutBlendedAnimationPoseData.GetCurve().Accumulate(AdditiveAnimationPoseData.GetCurve(), AdditivePose.Weight);
		UE::Anim::Attributes::AccumulateAttributes(AdditiveAnimationPoseData.GetAttributes(), OutBlendedAnimationPoseData.GetAttributes(), AdditivePose.Weight, AdditivePose.AdditiveType);
	}

	BlendProfileScratchData.Reset();
}

void FAnimInstanceProxy::SlotEvaluatePose(const FName& SlotNodeName, const FAnimationPoseData& SourceAnimationPoseData, float InSourceWeight, FAnimationPoseData& OutBlendedAnimationPoseData, float InBlendWeight, float InTotalNodeWeight)
{
	const FCompactPose& SourcePose = SourceAnimationPoseData.GetPose();
	const FBlendedCurve& SourceCurve = SourceAnimationPoseData.GetCurve();
	const UE::Anim::FStackAttributeContainer& SourceAttributes = SourceAnimationPoseData.GetAttributes();
	
	FCompactPose& BlendedPose = OutBlendedAnimationPoseData.GetPose();
	FBlendedCurve& BlendedCurve = OutBlendedAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& BlendedAttributes = OutBlendedAnimationPoseData.GetAttributes();

	// Accessing MontageInstances from this function is not safe (as this can be called during Parallel Anim Evaluation!
	// Any montage data you need to add should be part of MontageEvaluationData
	// nothing to blend, just get it out
	if (InBlendWeight <= ZERO_ANIMWEIGHT_THRESH)
	{
		BlendedPose = SourcePose;
		BlendedCurve = SourceCurve;
		BlendedAttributes = SourceAttributes;
		return;
	}

	// Check if we are blending a montage using a blend profile, if so take a special code path for this.
	for (const FMontageEvaluationState& EvalState : GetMontageEvaluationData())
	{
		if (EvalState.Montage.IsValid() && EvalState.ActiveBlendProfile && EvalState.Montage->IsValidSlot(SlotNodeName))
		{
			SlotEvaluatePoseWithBlendProfiles(SlotNodeName, SourceAnimationPoseData, InSourceWeight, OutBlendedAnimationPoseData, InBlendWeight);
			return;
		}
	}

	// Split our data into additive and non additive.
	FBlendProfileScratchData& BlendProfileScratchData = FBlendProfileScratchData::Get();
	TArray<FSlotEvaluationPose>& AdditivePoses = BlendProfileScratchData.Poses;
	TArray<FSlotEvaluationPose>& NonAdditivePoses = BlendProfileScratchData.AdditivePoses;
	TArray<float, TInlineAllocator<8>>& BlendingWeights = BlendProfileScratchData.BlendingWeights;
	TArray<const FCompactPose*, TInlineAllocator<8>>& BlendingPoses = BlendProfileScratchData.BlendingPoses;
	TArray<const FBlendedCurve*, TInlineAllocator<8>>& BlendingCurves = BlendProfileScratchData.BlendingCurves;
	TArray<const UE::Anim::FStackAttributeContainer*, TInlineAllocator<8>>& BlendingAttributes = BlendProfileScratchData.BlendingAttributes;
	check(BlendProfileScratchData.IsEmpty());

	// first pass we go through collect weights and valid montages. 
#if DEBUG_MONTAGEINSTANCE_WEIGHT
	float TotalWeight = 0.f;
#endif // DEBUG_MONTAGEINSTANCE_WEIGHT
	for (const FMontageEvaluationState& EvalState : GetMontageEvaluationData())
	{
		// If MontageEvaluationData is not valid anymore, pass-through AnimSlot.
		// This can happen if InitAnim pushes a RefreshBoneTransforms when not rendered,
		// with EVisibilityBasedAnimTickOption::OnlyTickMontagesWhenNotRendered set.
		if (!EvalState.Montage.IsValid())
		{
			BlendedPose = SourcePose;
			BlendedCurve = SourceCurve;
			BlendedAttributes = SourceAttributes;

			BlendProfileScratchData.Reset();
			return;
		}

		const UAnimMontage* const Montage = EvalState.Montage.Get();
		if (Montage->IsValidSlot(SlotNodeName))
		{
			FAnimTrack const* const AnimTrack = Montage->GetAnimationData(SlotNodeName);

			// Find out additive type for pose.
			EAdditiveAnimationType const AdditiveAnimType = AnimTrack->IsAdditive() 
				? (AnimTrack->IsRotationOffsetAdditive() ? AAT_RotationOffsetMeshSpace : AAT_LocalSpaceBase)
				: AAT_None;

			const float MontageWeight = EvalState.BlendInfo.GetBlendedValue();
			FSlotEvaluationPose NewPose(MontageWeight, AdditiveAnimType);
			
			// Bone array has to be allocated prior to calling GetPoseFromAnimTrack
			NewPose.Pose.SetBoneContainer(RequiredBones.Get());
			NewPose.Curve.InitFrom(*RequiredBones);

			// Extract pose from Track
			FAnimExtractContext ExtractionContext(static_cast<double>(EvalState.MontagePosition), Montage->HasRootMotion() && RootMotionMode != ERootMotionMode::NoRootMotionExtraction, EvalState.DeltaTimeRecord);
			ExtractionContext.InterpolationOverride = InterpolationOverride;

			FAnimationPoseData NewAnimationPoseData(NewPose);
			AnimTrack->GetAnimationPose(NewAnimationPoseData, ExtractionContext);

			// add montage curves 
			FBlendedCurve MontageCurve;
			MontageCurve.InitFrom(*RequiredBones);
			Montage->EvaluateCurveData(MontageCurve, ExtractionContext);
			NewPose.Curve.Combine(MontageCurve);

#if DEBUG_MONTAGEINSTANCE_WEIGHT
			TotalWeight += MontageWeight;
#endif // DEBUG_MONTAGEINSTANCE_WEIGHT

			if (AdditiveAnimType == AAT_None)
			{
				NonAdditivePoses.Add(FSlotEvaluationPose(MoveTemp(NewPose)));
			}
			else
			{
				AdditivePoses.Add(FSlotEvaluationPose(MoveTemp(NewPose)));
			}

#if ANIM_TRACE_ENABLED
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel))
			{
				FAnimationBaseContext Context;
				Context.AnimInstanceProxy = this;

				const float ClampedTime = FMath::Clamp(ExtractionContext.CurrentTime, 0.f, AnimTrack->GetLength());

				if (const FAnimSegment* const AnimSegment = AnimTrack->GetSegmentAtTime(ClampedTime))
				{
					float CurrentTime = 0;
					FAnimTickRecord TickRecord;
					TickRecord.SourceAsset = AnimSegment->GetAnimationData(ClampedTime, CurrentTime);
					TickRecord.PlayRateMultiplier = AnimSegment->AnimPlayRate;
					TickRecord.TimeAccumulator = &CurrentTime;
					TickRecord.DeltaTimeRecord = &ExtractionContext.DeltaTimeRecord;
					TickRecord.EffectiveBlendWeight = MontageWeight;
					TickRecord.SourceProxy = Context.AnimInstanceProxy;
					
					TRACE_ANIM_TICK_RECORD(Context, TickRecord);
				}
			}
#endif
		} // if IsValidSlot
	} // for all MontageEvaluationData

	// allocate for blending
	// If source has any weight, add it to the blend array.
	float const SourceWeight = FMath::Clamp<float>(InSourceWeight, 0.f, 1.f);

#if DEBUG_MONTAGEINSTANCE_WEIGHT
	ensure (FMath::IsNearlyEqual(InTotalNodeWeight, TotalWeight, UE_KINDA_SMALL_NUMBER));
#endif // DEBUG_MONTAGEINSTANCE_WEIGHT
	ensure (InTotalNodeWeight > ZERO_ANIMWEIGHT_THRESH);
	if (InTotalNodeWeight > (1.f + ZERO_ANIMWEIGHT_THRESH))
	{
		// Re-normalize additive poses
		for (int32 Index = 0; Index < AdditivePoses.Num(); Index++)
		{
			AdditivePoses[Index].Weight /= InTotalNodeWeight;
		}
		// Re-normalize non-additive poses
		for (int32 Index = 0; Index < NonAdditivePoses.Num(); Index++)
		{
			NonAdditivePoses[Index].Weight /= InTotalNodeWeight;
		}
	}

	// Make sure we have at least one montage here.
	ensure((AdditivePoses.Num() > 0) || (NonAdditivePoses.Num() > 0));

	// Second pass, blend non additive poses together
	{
		// If we're only playing additive animations, just copy source for base pose.
		if (NonAdditivePoses.Num() == 0)
		{
			// If the source weight is 0, SourcePose will be uninitialized and cannot be used
			if (InSourceWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				BlendedPose = SourcePose;
				BlendedCurve = SourceCurve;
				BlendedAttributes = SourceAttributes;
			}
			else
			{
				BlendedPose.ResetToRefPose(); 
			}
		}		
		else // Otherwise we need to blend non additive poses together
		{
			const int32 NumPoses = NonAdditivePoses.Num() + ((SourceWeight > ZERO_ANIMWEIGHT_THRESH) ? 1 : 0);
			BlendingPoses.AddUninitialized(NumPoses);
			BlendingWeights.AddUninitialized(NumPoses);
			BlendingCurves.AddUninitialized(NumPoses);
			BlendingAttributes.AddUninitialized(NumPoses);

			for (int32 Index = 0; Index < NonAdditivePoses.Num(); Index++)
			{
				BlendingPoses[Index] = &NonAdditivePoses[Index].Pose;
				BlendingCurves[Index] = &NonAdditivePoses[Index].Curve;
				BlendingAttributes[Index] = &NonAdditivePoses[Index].Attributes;
				BlendingWeights[Index] = NonAdditivePoses[Index].Weight;
			}

			if (SourceWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				const int32 SourceIndex = NumPoses - 1;
				BlendingPoses[SourceIndex] = &SourcePose;
				BlendingCurves[SourceIndex] = &SourceCurve;
				BlendingAttributes[SourceIndex] = &SourceAttributes;
				BlendingWeights[SourceIndex] = SourceWeight;
			}

			// Blend all montages.
			FAnimationRuntime::BlendPosesTogetherIndirect(BlendingPoses, BlendingCurves, BlendingAttributes, BlendingWeights, OutBlendedAnimationPoseData);
		}
	}

	// Third pass, layer on weighted additive poses.
	if (AdditivePoses.Num() > 0)
	{
		for (int32 Index = 0; Index < AdditivePoses.Num(); Index++)
		{
			FSlotEvaluationPose& AdditivePose = AdditivePoses[Index];
			const FAnimationPoseData AdditiveAnimationPoseData(AdditivePose);
			FAnimationRuntime::AccumulateAdditivePose(OutBlendedAnimationPoseData, AdditiveAnimationPoseData, AdditivePose.Weight, AdditivePose.AdditiveType);
		}
	}

	BlendProfileScratchData.Reset();
}

//to debug montage weight
#define DEBUGMONTAGEWEIGHT 0

void FAnimInstanceProxy::GetSlotWeight(const FName& SlotNodeName, float& out_SlotNodeWeight, float& out_SourceWeight, float& out_TotalNodeWeight) const
{
	// node total weight 
	float NewSlotNodeWeight = 0.f;
	// this is required to track, because it will be 1-SourceWeight
	// if additive, it can be applied more
	float NonAdditiveTotalWeight = 0.f;

#if DEBUGMONTAGEWEIGHT
	float TotalDesiredWeight = 0.f;
#endif

	// first get all the montage instance weight this slot node has
	for (const FMontageEvaluationState& EvalState : GetMontageEvaluationData())
	{
		if (EvalState.Montage.IsValid())
		{
			const UAnimMontage* const Montage = EvalState.Montage.Get();
			if (Montage->IsValidSlot(SlotNodeName))
			{
				const float MontageWeight = EvalState.BlendInfo.GetBlendedValue();
				NewSlotNodeWeight += MontageWeight;
				if (!Montage->IsValidAdditiveSlot(SlotNodeName))
				{
					NonAdditiveTotalWeight += MontageWeight;
				}

#if DEBUGMONTAGEWEIGHT			
				TotalDesiredWeight += EvalState->DesiredWeight;
#endif
#if ENABLE_ANIM_LOGGING
				UE_LOGF(LogAnimation, Verbose, "GetSlotWeight : Owner: %ls, AnimMontage: %ls,  (DesiredWeight:%0.2f, Weight:%0.2f)",
							*GetActorName(), *EvalState.Montage->GetName(), EvalState.BlendInfo.GetDesiredValue(), MontageWeight);
#endif
			}
		}
	}

	// save the total node weight, it can be more than 1
	// we need this so that when we eval, we normalized by this weight
	// calculating there can cause inconsistency if some data changes
	out_TotalNodeWeight = NewSlotNodeWeight;

	// this can happen when it's blending in OR when newer animation comes in with shorter blendtime
	// say #1 animation was blending out time with current blendtime 1.0 #2 animation was blending in with 1.0 (old) but got blend out with new blendtime 0.2f
	// #3 animation was blending in with the new blendtime 0.2f, you'll have sum of #1, 2, 3 exceeds 1.f
	if (NewSlotNodeWeight > 1.f)
	{
		// you don't want to change weight of montage instance since it can play multiple slots
		// if you change one, it will apply to all slots in that montage
		// instead we should renormalize when we eval
		// this should happen in the eval phase
		NonAdditiveTotalWeight /= NewSlotNodeWeight;
		// since we normalized, we reset
		NewSlotNodeWeight = 1.f;
	}
#if DEBUGMONTAGEWEIGHT
	else if (TotalDesiredWeight == 1.f && TotalSum < 1.f - ZERO_ANIMWEIGHT_THRESH)
	{
		// this can happen when it's blending in OR when newer animation comes in with longer blendtime
		// say #1 animation was blending out time with current blendtime 0.2 #2 animation was blending in with 0.2 (old) but got blend out with new blendtime 1.f
		// #3 animation was blending in with the new blendtime 1.f, you'll have sum of #1, 2, 3 doesn't meet 1.f
		UE_LOGF(LogAnimation, Warning, "[%ls] Montage has less weight. Blending in?(%f)", *SlotNodeName.ToString(), TotalSum);
	}
#endif

	out_SlotNodeWeight = NewSlotNodeWeight;
	out_SourceWeight = 1.f - NonAdditiveTotalWeight;
}

const FMontageEvaluationState* FAnimInstanceProxy::GetActiveMontageEvaluationState() const
{
	// Start from end, as most recent instances are added at the end of the queue.
	int32 const NumInstances = GetMontageEvaluationData().Num();
	for (int32 InstanceIndex = NumInstances - 1; InstanceIndex >= 0; InstanceIndex--)
	{
		const FMontageEvaluationState& EvaluationData = GetMontageEvaluationData()[InstanceIndex];
		if (EvaluationData.bIsActive)
		{
			return &EvaluationData;
		}
	}

	return nullptr;
}


TMap<FName, FInertializationRequest>& FAnimInstanceProxy::GetSlotGroupInertializationRequestDataMap()
{
	if (bUseMainInstanceMontageEvaluationData && GetMainInstanceProxy())
	{
		return GetMainInstanceProxy()->SlotGroupInertializationRequestDataMap;
	}

	return SlotGroupInertializationRequestDataMap;
}

void FAnimInstanceProxy::GatherDebugData(FNodeDebugData& DebugData)
{
	GatherDebugData_WithRoot(DebugData, RootNode, NAME_AnimGraph);
}

void FAnimInstanceProxy::GatherDebugData_WithRoot(FNodeDebugData& DebugData, FAnimNode_Base* InRootNode, FName InLayerName)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/Graph"));

	// Gather debug data for Root Node
	if(InRootNode != nullptr)
	{
		 InRootNode->GatherDebugData(DebugData); 
	}

	// Gather debug data for Cached Poses.
	if(TArray<FAnimNode_SaveCachedPose*>* SavedPoseQueue = SavedPoseQueueMap.Find(InLayerName))
	{
		for (FAnimNode_SaveCachedPose* PoseNode : *SavedPoseQueue)
		{
			PoseNode->GatherDebugData(DebugData);
		}
	}
}

#if ENABLE_ANIM_DRAW_DEBUG

void FAnimInstanceProxy::AnimDrawDebugOnScreenMessage(const FString& DebugMessage, const FColor& Color, const FVector2D& TextScale, ESceneDepthPriorityGroup DepthPriority)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::OnScreenMessage;
	DrawDebugItem.Message = DebugMessage;
	DrawDebugItem.Color = Color;
	DrawDebugItem.TextScale = TextScale;
	DrawDebugItem.DepthPriority = DepthPriority;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugInWorldMessage(const FString& DebugMessage, const FVector& TextLocation, const FColor& Color, float TextScale)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::InWorldMessage;
	DrawDebugItem.Message = DebugMessage;
	DrawDebugItem.StartLoc = TextLocation;
	DrawDebugItem.Color = Color;
	DrawDebugItem.TextScale.X = TextScale;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugDirectionalArrow(const FVector& LineStart, const FVector& LineEnd, float ArrowSize, const FColor& Color, bool bPersistentLines, float LifeTime, float Thickness, ESceneDepthPriorityGroup DepthPriority)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::DirectionalArrow;
	DrawDebugItem.StartLoc = LineStart;
	DrawDebugItem.EndLoc = LineEnd;
	DrawDebugItem.Size = ArrowSize;
	DrawDebugItem.Color = Color;
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.Thickness = Thickness;
	DrawDebugItem.DepthPriority = DepthPriority;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugSphere(const FVector& Center, float Radius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, float Thickness, ESceneDepthPriorityGroup DepthPriority)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::Sphere;
	DrawDebugItem.Center = Center;
	DrawDebugItem.Radius = Radius;
	DrawDebugItem.Segments = Segments;
	DrawDebugItem.Color = Color;
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.Thickness = Thickness;
	DrawDebugItem.DepthPriority = DepthPriority;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugCoordinateSystem(FVector const& AxisLoc, FRotator const& AxisRot, float Scale, bool bPersistentLines, float LifeTime, float Thickness, ESceneDepthPriorityGroup DepthPriority)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::CoordinateSystem;
	DrawDebugItem.StartLoc = AxisLoc;
	DrawDebugItem.Rotation = AxisRot;
	DrawDebugItem.Size = Scale;
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.Thickness = Thickness;
	DrawDebugItem.DepthPriority = DepthPriority;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugLine(const FVector& StartLoc, const FVector& EndLoc, const FColor& Color, bool bPersistentLines, float LifeTime, float Thickness, ESceneDepthPriorityGroup DepthPriority)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::Line;
	DrawDebugItem.StartLoc = StartLoc;
	DrawDebugItem.EndLoc = EndLoc;
	DrawDebugItem.Color = Color;
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.Thickness = Thickness;
	DrawDebugItem.DepthPriority = DepthPriority;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugPlane(const FTransform& BaseTransform, float Radii, const FColor& Color, bool bPersistentLines /*= false*/, float LifeTime /*= -1.f*/, float Thickness /*= 0.f*/, ESceneDepthPriorityGroup DepthPriority)
{
	// just draw two triangle from [-Radii,-Radii] to [Radii, Radii]
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::Line;
	DrawDebugItem.Color = Color;
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.Thickness = Thickness;
	DrawDebugItem.DepthPriority = DepthPriority;

	DrawDebugItem.StartLoc = BaseTransform.TransformPosition(FVector(-Radii, -Radii, 0));
	DrawDebugItem.EndLoc = BaseTransform.TransformPosition(FVector(-Radii, Radii, 0));
	QueuedDrawDebugItems.Add(DrawDebugItem);

	DrawDebugItem.StartLoc = BaseTransform.TransformPosition(FVector(-Radii, -Radii, 0));
	DrawDebugItem.EndLoc = BaseTransform.TransformPosition(FVector(Radii, -Radii, 0));
	QueuedDrawDebugItems.Add(DrawDebugItem);

	DrawDebugItem.StartLoc = BaseTransform.TransformPosition(FVector(-Radii, Radii, 0));
	DrawDebugItem.EndLoc = BaseTransform.TransformPosition(FVector(-Radii, Radii, 0));
	QueuedDrawDebugItems.Add(DrawDebugItem);

	DrawDebugItem.StartLoc = BaseTransform.TransformPosition(FVector(Radii, Radii, 0));
	DrawDebugItem.EndLoc = BaseTransform.TransformPosition(FVector(-Radii, Radii, 0));
	QueuedDrawDebugItems.Add(DrawDebugItem);

	DrawDebugItem.StartLoc = BaseTransform.TransformPosition(FVector(Radii, Radii, 0));
	DrawDebugItem.EndLoc = BaseTransform.TransformPosition(FVector(Radii, -Radii, 0));
	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugPoint(const FVector& Loc, float Size, const FColor& Color, bool bPersistentLines, float LifeTime, ESceneDepthPriorityGroup DepthPriority)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::Point;
	DrawDebugItem.StartLoc = Loc;
	DrawDebugItem.Color = Color;
	DrawDebugItem.Size = Size;
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.DepthPriority = DepthPriority;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugCircle(const FVector& Center, float Radius, int32 Segments, const FColor& Color, const FVector& UpVector, bool bPersistentLines, float LifeTime, ESceneDepthPriorityGroup DepthPriority, float Thickness)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::Circle;
	DrawDebugItem.Center = Center;
	DrawDebugItem.Radius = Radius;
	DrawDebugItem.Segments = Segments;
	DrawDebugItem.Color = Color;
	// We're using EndLoc as our direction unit vector.
	DrawDebugItem.EndLoc = UpVector.GetSafeNormal();
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.DepthPriority = DepthPriority;
	DrawDebugItem.Thickness = Thickness;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugCone(const FVector& Center, float Length, const FVector& Direction, float AngleWidth, float AngleHeight, int32 Segments, const FColor & Color, bool bPersistentLines, float LifeTime, ESceneDepthPriorityGroup DepthPriority, float Thickness)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::Cone;
	DrawDebugItem.Center = Center;
	DrawDebugItem.Length = Length;
	DrawDebugItem.AngleWidth = AngleWidth;
	DrawDebugItem.AngleHeight = AngleHeight;
	// We're using EndLoc as our direction unit vector.
	DrawDebugItem.Direction = Direction.GetSafeNormal();
	DrawDebugItem.Segments = Segments;
	DrawDebugItem.Color = Color;
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.DepthPriority = DepthPriority;
	DrawDebugItem.Thickness = Thickness;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

void FAnimInstanceProxy::AnimDrawDebugCapsule(const FVector& Center, float HalfHeight, float Radius, const FRotator& Rotation, const FColor& Color, bool bPersistentLines /*= false*/, float LifeTime /*= -1.f*/, float Thickness /*= 0.f*/, ESceneDepthPriorityGroup DepthPriority /*= SDPG_World*/)
{
	FQueuedDrawDebugItem DrawDebugItem;

	DrawDebugItem.ItemType = EDrawDebugItemType::Capsule;
	DrawDebugItem.Center = Center;
	DrawDebugItem.Size = HalfHeight;
	DrawDebugItem.Radius = Radius;
	DrawDebugItem.Rotation = Rotation;
	DrawDebugItem.Color = Color;
	DrawDebugItem.bPersistentLines = bPersistentLines;
	DrawDebugItem.LifeTime = LifeTime;
	DrawDebugItem.Thickness = Thickness;
	DrawDebugItem.DepthPriority = DepthPriority;

	QueuedDrawDebugItems.Add(DrawDebugItem);
}

#endif // ENABLE_ANIM_DRAW_DEBUG

float FAnimInstanceProxy::GetInstanceAssetPlayerLength(int32 AssetPlayerIndex) const
{
	if(const FAnimNode_AssetPlayerBase* PlayerNode = GetNodeFromIndex<FAnimNode_AssetPlayerBase>(AssetPlayerIndex))
	{
		return PlayerNode->GetCurrentAssetLength();
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetInstanceAssetPlayerTime(int32 AssetPlayerIndex) const
{
	if(const FAnimNode_AssetPlayerBase* PlayerNode = GetNodeFromIndex<FAnimNode_AssetPlayerBase>(AssetPlayerIndex))
	{
		return PlayerNode->GetCurrentAssetTimePlayRateAdjusted();
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetInstanceAssetPlayerTimeFraction(int32 AssetPlayerIndex) const
{
	if(const FAnimNode_AssetPlayerBase* PlayerNode = GetNodeFromIndex<FAnimNode_AssetPlayerBase>(AssetPlayerIndex))
	{
		float Length = PlayerNode->GetCurrentAssetLength();

		if(Length > 0.0f)
		{
			return PlayerNode->GetCurrentAssetTimePlayRateAdjusted() / Length;
		}
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetInstanceAssetPlayerTimeFromEndFraction(int32 AssetPlayerIndex) const
{
	if(const FAnimNode_AssetPlayerBase* PlayerNode = GetNodeFromIndex<FAnimNode_AssetPlayerBase>(AssetPlayerIndex))
	{
		float Length = PlayerNode->GetCurrentAssetLength();

		if(Length > 0.f)
		{
			return (Length - PlayerNode->GetCurrentAssetTimePlayRateAdjusted()) / Length;
		}
	}

	return 1.0f;
}

float FAnimInstanceProxy::GetInstanceAssetPlayerTimeFromEnd(int32 AssetPlayerIndex) const
{
	if(const FAnimNode_AssetPlayerBase* PlayerNode = GetNodeFromIndex<FAnimNode_AssetPlayerBase>(AssetPlayerIndex))
	{
		return PlayerNode->GetCurrentAssetLength() - PlayerNode->GetCurrentAssetTimePlayRateAdjusted();
	}

	return MAX_flt;
}

float FAnimInstanceProxy::GetInstanceMachineWeight(int32 MachineIndex) const
{
	if (const FAnimNode_StateMachine* MachineInstance = GetStateMachineInstance(MachineIndex))
	{
		return GetRecordedMachineWeight(MachineInstance->StateMachineIndexInClass);
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetInstanceStateWeight(int32 MachineIndex, int32 StateIndex) const
{
	if(const FAnimNode_StateMachine* MachineInstance = GetStateMachineInstance(MachineIndex))
	{
		return GetRecordedStateWeight(MachineInstance->StateMachineIndexInClass, StateIndex);
	}
	
	return 0.0f;
}

float FAnimInstanceProxy::GetInstanceCurrentStateElapsedTime(int32 MachineIndex) const
{
	if(const FAnimNode_StateMachine* MachineInstance = GetStateMachineInstance(MachineIndex))
	{
		return MachineInstance->GetCurrentStateElapsedTime();
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetInstanceTransitionCrossfadeDuration(int32 MachineIndex, int32 TransitionIndex) const
{
	if(const FAnimNode_StateMachine* MachineInstance = GetStateMachineInstance(MachineIndex))
	{
		if(MachineInstance->IsValidTransitionIndex(TransitionIndex))
		{
			return MachineInstance->GetTransitionInfo(TransitionIndex).CrossfadeDuration;
		}
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetInstanceTransitionTimeElapsed(int32 MachineIndex, int32 TransitionIndex) const
{
	if(const FAnimNode_StateMachine* MachineInstance = GetStateMachineInstance(MachineIndex))
	{
		if(MachineInstance->IsValidTransitionIndex(TransitionIndex))
		{
			for(const FAnimationActiveTransitionEntry& ActiveTransition : MachineInstance->ActiveTransitionArray)
			{
				if(ActiveTransition.SourceTransitionIndices.Contains(TransitionIndex))
				{
					return ActiveTransition.ElapsedTime;
				}
			}
		}
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetInstanceTransitionTimeElapsedFraction(int32 MachineIndex, int32 TransitionIndex) const
{
	if(const FAnimNode_StateMachine* MachineInstance = GetStateMachineInstance(MachineIndex))
	{
		if(MachineInstance->IsValidTransitionIndex(TransitionIndex))
		{
			for(const FAnimationActiveTransitionEntry& ActiveTransition : MachineInstance->ActiveTransitionArray)
			{
				if(ActiveTransition.SourceTransitionIndices.Contains(TransitionIndex))
				{
					return ActiveTransition.ElapsedTime / ActiveTransition.CrossfadeDuration;
				}
			}
		}
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetRelevantAnimTimeRemaining(int32 MachineIndex, int32 StateIndex) const
{
	if (const FAnimNode_StateMachine* StateMachine = GetStateMachineInstance(MachineIndex))
	{
		return StateMachine->GetRelevantAnimTimeRemaining(this, StateIndex);
	}

	return MAX_flt;
}

float FAnimInstanceProxy::GetRelevantAnimTimeRemainingFraction(int32 MachineIndex, int32 StateIndex) const
{
	if (const FAnimNode_StateMachine* StateMachine = GetStateMachineInstance(MachineIndex))
	{
		return StateMachine->GetRelevantAnimTimeRemainingFraction(this, StateIndex);
	}

	return 1.0f;
}

float FAnimInstanceProxy::GetRelevantAnimLength(int32 MachineIndex, int32 StateIndex) const
{
	if(const FAnimNode_AssetPlayerRelevancyBase* AssetPlayer = GetRelevantAssetPlayerInterfaceFromState(MachineIndex, StateIndex))
	{
		if(AssetPlayer->GetAnimAsset())
		{
			return AssetPlayer->GetCurrentAssetLength();
		}
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetRelevantAnimTime(int32 MachineIndex, int32 StateIndex) const
{
	if(const FAnimNode_AssetPlayerRelevancyBase* AssetPlayer = GetRelevantAssetPlayerInterfaceFromState(MachineIndex, StateIndex))
	{
		return AssetPlayer->GetCurrentAssetTimePlayRateAdjusted();
	}

	return 0.0f;
}

float FAnimInstanceProxy::GetRelevantAnimTimeFraction(int32 MachineIndex, int32 StateIndex) const
{
	if(const FAnimNode_AssetPlayerRelevancyBase* AssetPlayer = GetRelevantAssetPlayerInterfaceFromState(MachineIndex, StateIndex))
	{
		float Length = AssetPlayer->GetCurrentAssetLength();
		if(Length > 0.0f)
		{
			return AssetPlayer->GetCurrentAssetTimePlayRateAdjusted() / Length;
		}
	}

	return 0.0f;
}

TArray<FMontageEvaluationState>& FAnimInstanceProxy::GetMontageEvaluationData()
{
	if (bUseMainInstanceMontageEvaluationData && GetMainInstanceProxy())
	{
		return GetMainInstanceProxy()->MontageEvaluationData;
	}

	return MontageEvaluationData;
}

const TArray<FMontageEvaluationState>& FAnimInstanceProxy::GetMontageEvaluationData() const
{
	if (bUseMainInstanceMontageEvaluationData && GetMainInstanceProxy())
	{
		return GetMainInstanceProxy()->MontageEvaluationData;
	}

	return MontageEvaluationData;
}
bool FAnimInstanceProxy::WasAnimNotifyStateActiveInAnyState(TSubclassOf<UAnimNotifyState> AnimNotifyStateType) const
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent && NotifyEvent->NotifyStateClass 
			&& NotifyEvent->NotifyStateClass->IsA(AnimNotifyStateType))
		{
			return true;
		}
	}

	return false;
}

bool FAnimInstanceProxy::WasAnimNotifyStateActiveInStateMachine(int32 MachineIndex, TSubclassOf<UAnimNotifyState> AnimNotifyStateType) const
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent && NotifyEvent->NotifyStateClass && NotifyEvent->NotifyStateClass->IsA(AnimNotifyStateType))
		{
			return UAnimNotifyStateMachineInspectionLibrary::IsStateMachineInEventContext(Ref, MachineIndex);
		}
	}
	return false;
}

bool FAnimInstanceProxy::WasAnimNotifyStateActiveInSourceState(int32 MachineIndex, int32 StateIndex, TSubclassOf<UAnimNotifyState> AnimNotifyStateType) const
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent && NotifyEvent->NotifyStateClass && NotifyEvent->NotifyStateClass->IsA(AnimNotifyStateType))
		{
			return UAnimNotifyStateMachineInspectionLibrary::IsStateInStateMachineInEventContext(Ref, MachineIndex, StateIndex);
		}
	}
	return false;
}

bool FAnimInstanceProxy::WasAnimNotifyTriggeredInSourceState(int32 MachineIndex, int32 StateIndex, TSubclassOf<UAnimNotify> AnimNotifyType) const
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent && NotifyEvent->Notify && NotifyEvent->Notify->IsA(AnimNotifyType))
		{
			return UAnimNotifyStateMachineInspectionLibrary::IsStateInStateMachineInEventContext(Ref, MachineIndex, StateIndex);
		}
	}
	return false;
}

bool FAnimInstanceProxy::WasAnimNotifyNameTriggeredInSourceState(int32 MachineIndex, int32 StateIndex, FName NotifyName) const
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent)
		{
			FName LookupName = NotifyEvent->NotifyName;
			if (Ref.GetMirrorDataTable())
			{
				const FName* MirroredName = Ref.GetMirrorDataTable()->AnimNotifyToMirrorAnimNotifyMap.Find(LookupName);
				if (MirroredName)
				{
					LookupName = *MirroredName; 
				}
			}
			if(LookupName == NotifyName)
			{
				return UAnimNotifyStateMachineInspectionLibrary::IsStateInStateMachineInEventContext(Ref, MachineIndex, StateIndex);
			}
		}
	}
	return false;
}

bool FAnimInstanceProxy::WasAnimNotifyTriggeredInStateMachine(int32 MachineIndex,TSubclassOf<UAnimNotify> AnimNotifyType) const
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent && NotifyEvent->Notify && NotifyEvent->Notify->IsA(AnimNotifyType))
		{
			return UAnimNotifyStateMachineInspectionLibrary::IsStateMachineInEventContext(Ref, MachineIndex);
		}
	}
	return false;
}

bool FAnimInstanceProxy::WasAnimNotifyTriggeredInAnyState(TSubclassOf<UAnimNotify> AnimNotifyType) const
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent && NotifyEvent->Notify && NotifyEvent->Notify->IsA(AnimNotifyType))
		{
			return true;
		}
	}
	return false;
}

bool FAnimInstanceProxy::WasAnimNotifyNameTriggeredInAnyState(FName NotifyName) const
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent)
		{
			FName LookupName = NotifyEvent->NotifyName;
			if (Ref.GetMirrorDataTable())
			{
				const FName* MirroredName = Ref.GetMirrorDataTable()->AnimNotifyToMirrorAnimNotifyMap.Find(LookupName);
				if (MirroredName)
				{
					LookupName = *MirroredName; 
				}
			}
			if(LookupName == NotifyName)
			{
				return true;
			}
		}
	}
	return false; 
}

bool FAnimInstanceProxy::WasAnimNotifyNameTriggeredInStateMachine(int32 MachineIndex, FName NotifyName)
{
	for (const FAnimNotifyEventReference& Ref : ActiveAnimNotifiesSinceLastTick)
	{
		const FAnimNotifyEvent* NotifyEvent = Ref.GetNotify();
		if (NotifyEvent)
		{
			FName LookupName = NotifyEvent->NotifyName;
			if (Ref.GetMirrorDataTable())
			{
				const FName* MirroredName = Ref.GetMirrorDataTable()->AnimNotifyToMirrorAnimNotifyMap.Find(LookupName);
				if (MirroredName)
				{
					LookupName = *MirroredName; 
				}
			}
			if(LookupName == NotifyName)
			{
				return UAnimNotifyStateMachineInspectionLibrary::IsStateMachineInEventContext(Ref, MachineIndex);
			}
		}
	}
	return false;
}

bool FAnimInstanceProxy::RequestTransitionEvent(const FName& EventName, const double RequestTimeout, const ETransitionRequestQueueMode& QueueMode, const ETransitionRequestOverwriteMode& OverwriteMode)
{
	FTransitionEvent NewTransitionEvent(EventName, RequestTimeout, QueueMode, OverwriteMode);

	if (!NewTransitionEvent.IsValidRequest())
	{
		return false;
	}
	
	ForEachStateMachine([NewTransitionEvent](FAnimNode_StateMachine& StateMachine)
	{
		StateMachine.RequestTransitionEvent(NewTransitionEvent);
	});

	return true;
}

bool FAnimInstanceProxy::QueryTransitionEvent(int32 NodeIndex, int32 TransitionIndex, const FName& EventName) const
{
	const FAnimNode_StateMachine* StateMachine = GetStateMachineInstance(NodeIndex);
	if (StateMachine)
	{
		return StateMachine->QueryTransitionEvent(TransitionIndex, EventName);
	}
	return false;
}

void FAnimInstanceProxy::ClearTransitionEvents(const FName& EventName)
{
	ForEachStateMachine([EventName](FAnimNode_StateMachine& StateMachine)
	{
		StateMachine.ClearTransitionEvents(EventName);
	});
}

void FAnimInstanceProxy::ClearAllTransitionEvents()
{
	ForEachStateMachine([](FAnimNode_StateMachine& StateMachine)
	{
		StateMachine.ClearAllTransitionEvents();
	});
}

bool FAnimInstanceProxy::QueryAndMarkTransitionEvent(int32 NodeIndex, int32 TransitionIndex, const FName& EventName)
{
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();

		FStructProperty* Property = AnimNodeProperties[AnimNodeProperties.Num() - 1 - NodeIndex];
		if (Property && Property->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
		{
			FAnimNode_StateMachine* StateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
			if (StateMachine)
			{
				return StateMachine->QueryAndMarkTransitionEvent(TransitionIndex, EventName);
			}
		}
	}

	return false;
}

const FAnimNode_AssetPlayerRelevancyBase* FAnimInstanceProxy::GetRelevantAssetPlayerInterfaceFromState(int32 NodeIndex, int32 StateIndex) const
{
	if (const FAnimNode_StateMachine* StateMachine = GetStateMachineInstance(NodeIndex))
	{
		return StateMachine->GetRelevantAssetPlayerInterfaceFromState(this, StateMachine->GetStateInfo(StateIndex));
	}

	return nullptr;
}

const FAnimNode_StateMachine* FAnimInstanceProxy::GetStateMachineInstance(int32 NodeIndex) const
{
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		if ((NodeIndex >= 0) && (NodeIndex < AnimNodeProperties.Num()))
		{
			const int32 InstancePropertyIndex = AnimNodeProperties.Num() - 1 - NodeIndex;

			FStructProperty* MachineInstanceProperty = AnimNodeProperties[InstancePropertyIndex];
			if(MachineInstanceProperty->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
			{
				return MachineInstanceProperty->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
			}
		}
	}

	return nullptr;
}

void FAnimInstanceProxy::AddNativeTransitionBinding(const FName& MachineName, const FName& PrevStateName, const FName& NextStateName, const FCanTakeTransition& NativeTransitionDelegate, const FName& TransitionName)
{
	NativeTransitionBindings.Add(FNativeTransitionBinding(MachineName, PrevStateName, NextStateName, NativeTransitionDelegate, TransitionName));
}

bool FAnimInstanceProxy::HasNativeTransitionBinding(const FName& MachineName, const FName& PrevStateName, const FName& NextStateName, FName& OutBindingName)
{
	for(const auto& Binding : NativeTransitionBindings)
	{
		if(Binding.MachineName == MachineName && Binding.PreviousStateName == PrevStateName && Binding.NextStateName == NextStateName)
		{
#if WITH_EDITORONLY_DATA
				OutBindingName = Binding.TransitionName;
#else
			OutBindingName = NAME_None;
#endif
			return true;
		}
	}

	return false;
}

void FAnimInstanceProxy::AddNativeStateEntryBinding(const FName& MachineName, const FName& StateName, const FOnGraphStateChanged& NativeEnteredDelegate, const FName& BindingName)
{
	NativeStateEntryBindings.Add(FNativeStateBinding(MachineName, StateName, NativeEnteredDelegate, BindingName));
}
	
bool FAnimInstanceProxy::HasNativeStateEntryBinding(const FName& MachineName, const FName& StateName, FName& OutBindingName)
{
	for(const auto& Binding : NativeStateEntryBindings)
	{
		if(Binding.MachineName == MachineName && Binding.StateName == StateName)
		{
#if WITH_EDITORONLY_DATA
			OutBindingName = Binding.BindingName;
#else
			OutBindingName = NAME_None;
#endif
			return true;
		}
	}

	return false;
}

void FAnimInstanceProxy::AddNativeStateExitBinding(const FName& MachineName, const FName& StateName, const FOnGraphStateChanged& NativeExitedDelegate, const FName& BindingName)
{
	NativeStateExitBindings.Add(FNativeStateBinding(MachineName, StateName, NativeExitedDelegate, BindingName));
}

bool FAnimInstanceProxy::HasNativeStateExitBinding(const FName& MachineName, const FName& StateName, FName& OutBindingName)
{
	for(const auto& Binding : NativeStateExitBindings)
	{
		if(Binding.MachineName == MachineName && Binding.StateName == StateName)
		{
#if WITH_EDITORONLY_DATA
			OutBindingName = Binding.BindingName;
#else
			OutBindingName = NAME_None;
#endif
			return true;
		}
	}

	return false;
}

void FAnimInstanceProxy::BindNativeDelegates()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// if we have no root node, we are usually in error so early out
	if(RootNode == nullptr)
	{
		return;
	}

	auto ForEachStateLambda = [&](IAnimClassInterface* InAnimClassInterface, const FName& MachineName, const FName& StateName, TFunctionRef<void(FAnimNode_StateMachine*, const FBakedAnimationState&, int32)> Predicate)
	{
		for (const FStructProperty* Property : InAnimClassInterface->GetAnimNodeProperties())
		{
			if(Property && Property->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
			{
				FAnimNode_StateMachine* StateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
				if(StateMachine)
				{
					const FBakedAnimationStateMachine* MachineDescription = GetMachineDescription(InAnimClassInterface, StateMachine);
					if(MachineDescription && MachineName == MachineDescription->MachineName)
					{
						// check each state transition for a match
						int32 StateIndex = 0;
						for(const FBakedAnimationState& State : MachineDescription->States)
						{
							if(State.StateName == StateName)
							{
								Predicate(StateMachine, State, StateIndex);
							}
							StateIndex++;
						}
					}
				}
			}
		}
	};

	if (AnimClassInterface)
	{
		// transition delegates
		for(const auto& Binding : NativeTransitionBindings)
		{
			ForEachStateLambda(AnimClassInterface, Binding.MachineName, Binding.PreviousStateName,
				[&](FAnimNode_StateMachine* StateMachine, const FBakedAnimationState& State, int32 StateIndex)
				{
					for(const FBakedStateExitTransition& TransitionExit : State.Transitions)
					{
						if(TransitionExit.CanTakeDelegateIndex != INDEX_NONE)
						{
							// In case the state machine hasn't been initialized, we need to re-get the desc
							const FBakedAnimationStateMachine* MachineDesc = GetMachineDescription(AnimClassInterface, StateMachine);
							const FAnimationTransitionBetweenStates& Transition = MachineDesc->Transitions[TransitionExit.TransitionIndex];
							const FBakedAnimationState& BakedState = MachineDesc->States[Transition.NextState];

							if (BakedState.StateName == Binding.NextStateName)
							{
								FAnimNode_TransitionResult* ResultNode = GetNodeFromPropertyIndex<FAnimNode_TransitionResult>(AnimInstanceObject, AnimClassInterface, TransitionExit.CanTakeDelegateIndex);
								if(ResultNode)
								{
									ResultNode->NativeTransitionDelegate = Binding.NativeTransitionDelegate;
								}
							}
						}
					}
				});
		}

		// state entry delegates
		for(const auto& Binding : NativeStateEntryBindings)
		{
			ForEachStateLambda(AnimClassInterface, Binding.MachineName, Binding.StateName,
				[&](FAnimNode_StateMachine* StateMachine, const FBakedAnimationState& State, int32 StateIndex)
				{
					// allocate enough space for all our states we need so far
					StateMachine->OnGraphStatesEntered.SetNum(FMath::Max(StateIndex + 1, StateMachine->OnGraphStatesEntered.Num()));
					StateMachine->OnGraphStatesEntered[StateIndex] = Binding.NativeStateDelegate;
				});
		}

		// state exit delegates
		for(const auto& Binding : NativeStateExitBindings)
		{
			ForEachStateLambda(AnimClassInterface, Binding.MachineName, Binding.StateName,
				[&](FAnimNode_StateMachine* StateMachine, const FBakedAnimationState& State, int32 StateIndex)
				{
					// allocate enough space for all our states we need so far
					StateMachine->OnGraphStatesExited.SetNum(FMath::Max(StateIndex + 1, StateMachine->OnGraphStatesExited.Num()));
					StateMachine->OnGraphStatesExited[StateIndex] = Binding.NativeStateDelegate;
				});
		}
	}
}

const FBakedAnimationStateMachine* FAnimInstanceProxy::GetMachineDescription(IAnimClassInterface* AnimBlueprintClass, const FAnimNode_StateMachine* MachineInstance)
{
	const TArray<FBakedAnimationStateMachine>& BakedStateMachines = AnimBlueprintClass->GetBakedStateMachines();
	return BakedStateMachines.IsValidIndex(MachineInstance->StateMachineIndexInClass) ? &(BakedStateMachines[MachineInstance->StateMachineIndexInClass]) : nullptr;
}

const FAnimNode_StateMachine* FAnimInstanceProxy::GetStateMachineInstanceFromName(FName MachineName) const
{
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		for (int32 MachineIndex = 0; MachineIndex < AnimNodeProperties.Num(); MachineIndex++)
		{
			FStructProperty* Property = AnimNodeProperties[AnimNodeProperties.Num() - 1 - MachineIndex];
			if (Property && Property->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
			{
				const FAnimNode_StateMachine* StateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
				if (StateMachine)
				{
					if (const FBakedAnimationStateMachine* MachineDescription = GetMachineDescription(AnimClassInterface, StateMachine))
					{
						if (MachineDescription->MachineName == MachineName)
						{
							return StateMachine;
						}
					}
				}
			}
		}
	}

	return nullptr;
}

const FBakedAnimationStateMachine* FAnimInstanceProxy::GetStateMachineInstanceDesc(FName MachineName) const
{
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		for (int32 MachineIndex = 0; MachineIndex < AnimNodeProperties.Num(); MachineIndex++)
		{
			FStructProperty* Property = AnimNodeProperties[AnimNodeProperties.Num() - 1 - MachineIndex];
			if(Property && Property->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
			{
				FAnimNode_StateMachine* StateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
				if(StateMachine)
				{
					if (const FBakedAnimationStateMachine* MachineDescription = GetMachineDescription(AnimClassInterface, StateMachine))
					{
						if(MachineDescription->MachineName == MachineName)
						{
							return MachineDescription;
						}
					}
				}
			}
		}
	}

	return nullptr;
}

int32 FAnimInstanceProxy::GetStateMachineIndex(FName MachineName) const
{
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		for (int32 MachineIndex = 0; MachineIndex < AnimNodeProperties.Num(); MachineIndex++)
		{
			FStructProperty* Property = AnimNodeProperties[AnimNodeProperties.Num() - 1 - MachineIndex];
			if(Property && Property->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
			{
				const FAnimNode_StateMachine* StateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
				if(StateMachine)
				{
					if (const FBakedAnimationStateMachine* MachineDescription = GetMachineDescription(AnimClassInterface, StateMachine))
					{
						if(MachineDescription->MachineName == MachineName)
						{
							return MachineIndex;
						}
					}
				}
			}
		}
	}

	return INDEX_NONE;
}

int32 FAnimInstanceProxy::GetStateMachineIndex(FAnimNode_StateMachine* StateMachine) const
{
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		for (int32 NodeIndex = 0; NodeIndex < AnimNodeProperties.Num(); NodeIndex++)
		{
			FStructProperty* Property = AnimNodeProperties[AnimNodeProperties.Num() - 1 - NodeIndex];
			if (Property && Property->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
			{
				FAnimNode_StateMachine* CurStateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
				if (CurStateMachine == StateMachine)
				{
					return NodeIndex;
				}
			}
		}
	}
	return INDEX_NONE; 	
}

void FAnimInstanceProxy::GetStateMachineIndexAndDescription(FName InMachineName, int32& OutNodeIndex, const FBakedAnimationStateMachine** OutMachineDescription) const
{
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		for (int32 NodeIndex = 0; NodeIndex < AnimNodeProperties.Num(); NodeIndex++)
		{
			FStructProperty* Property = AnimNodeProperties[AnimNodeProperties.Num() - 1 - NodeIndex];
			if (Property && Property->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
			{
				const FAnimNode_StateMachine* StateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
				if (StateMachine)
				{
					if (const FBakedAnimationStateMachine* MachineDescription = GetMachineDescription(AnimClassInterface, StateMachine))
					{
						if (MachineDescription->MachineName == InMachineName)
						{
							OutNodeIndex = NodeIndex;
							if (OutMachineDescription)
							{
								*OutMachineDescription = MachineDescription;
							}
							return;
						}
					}
				}
			}
		}
	}

	OutNodeIndex = INDEX_NONE;
	if (OutMachineDescription)
	{
		*OutMachineDescription = nullptr;
	}
}

int32 FAnimInstanceProxy::GetInstanceAssetPlayerIndex(FName MachineName, FName StateName, FName AssetName) const
{
	if (AnimClassInterface)
	{
		if(const FBakedAnimationStateMachine* MachineDescription = GetStateMachineInstanceDesc(MachineName))
		{
			const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
			for(int32 StateIndex = 0; StateIndex < MachineDescription->States.Num(); StateIndex++)
			{
				const FBakedAnimationState& State = MachineDescription->States[StateIndex];
				if(State.StateName == StateName)
				{
					for(int32 PlayerIndex = 0; PlayerIndex < State.PlayerNodeIndices.Num(); PlayerIndex++)
					{
						checkSlow(State.PlayerNodeIndices[PlayerIndex] < AnimNodeProperties.Num());
						FStructProperty* AssetPlayerProperty = AnimNodeProperties[AnimNodeProperties.Num() - 1 - State.PlayerNodeIndices[PlayerIndex]];
						if(AssetPlayerProperty && AssetPlayerProperty->Struct->IsChildOf(FAnimNode_AssetPlayerBase::StaticStruct()))
						{
							const FAnimNode_AssetPlayerBase* AssetPlayer = AssetPlayerProperty->ContainerPtrToValuePtr<FAnimNode_AssetPlayerBase>(AnimInstanceObject);
							if(AssetPlayer)
							{
								if(AssetName == NAME_None || AssetPlayer->GetAnimAsset()->GetFName() == AssetName)
								{
									return State.PlayerNodeIndices[PlayerIndex];
								}
							}
						}
					}
				}
			}
		}
	}

	return INDEX_NONE;
}

float FAnimInstanceProxy::GetRecordedMachineWeight(const int32 InMachineClassIndex) const
{
	return MachineWeightArrays[GetBufferReadIndex()][InMachineClassIndex];
}

void FAnimInstanceProxy::RecordMachineWeight(const int32 InMachineClassIndex, const float InMachineWeight)
{
	MachineWeightArrays[GetBufferWriteIndex()][InMachineClassIndex] = InMachineWeight;
}

float FAnimInstanceProxy::GetRecordedStateWeight(const int32 InMachineClassIndex, const int32 InStateIndex) const
{
	const int32* BaseIndexPtr = StateMachineClassIndexToWeightOffset.Find(InMachineClassIndex);

	if(BaseIndexPtr)
	{
		const int32 StateIndex = *BaseIndexPtr + InStateIndex;
		return StateWeightArrays[GetBufferReadIndex()][StateIndex];
	}

	return 0.0f;
}

void FAnimInstanceProxy::RecordStateWeight(const int32 InMachineClassIndex, const int32 InStateIndex, const float InStateWeight, const float InElapsedTime)
{
	const int32* BaseIndexPtr = StateMachineClassIndexToWeightOffset.Find(InMachineClassIndex);

	if(BaseIndexPtr)
	{
		const int32 StateIndex = *BaseIndexPtr + InStateIndex;
		StateWeightArrays[GetBufferWriteIndex()][StateIndex] = InStateWeight;
	}

#if WITH_EDITORONLY_DATA
	if (FAnimBlueprintDebugData* DebugData = GetAnimBlueprintDebugData())
	{
		DebugData->RecordStateData(InMachineClassIndex, InStateIndex, InStateWeight, InElapsedTime);
	}
#endif
}

void FAnimInstanceProxy::ResetDynamics(ETeleportType InTeleportType)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/Graph"));

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	for(FAnimNode_Base* Node : DynamicResetNodes)
	{
		Node->ResetDynamics(InTeleportType);
	}
}

void FAnimInstanceProxy::ResetDynamics()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/Graph"));

	ResetDynamics(ETeleportType::ResetPhysics);
}

void FAnimInstanceProxy::TraceMontageEvaluationData(const FAnimationUpdateContext& InContext, const FName& InSlotName)
{
#if ANIM_TRACE_ENABLED
	for (const FMontageEvaluationState& MontageEvaluationState : GetMontageEvaluationData())
	{
		if (MontageEvaluationState.Montage != nullptr && MontageEvaluationState.Montage->IsValidSlot(InSlotName))
		{
			if (const FAnimTrack* const Track = MontageEvaluationState.Montage->GetAnimationData(InSlotName))
			{
				if (const FAnimSegment* const Segment = Track->GetSegmentAtTime(MontageEvaluationState.MontagePosition))
				{
					float CurrentAnimPos;
					if (UAnimSequenceBase* Anim = Segment->GetAnimationData(MontageEvaluationState.MontagePosition, CurrentAnimPos))
					{
						TRACE_ANIM_NODE_VALUE(InContext, TEXT("Montage"), MontageEvaluationState.Montage.Get());
						TRACE_ANIM_NODE_VALUE(InContext, TEXT("Sequence"), Anim);
						TRACE_ANIM_NODE_VALUE(InContext, TEXT("Sequence Playback Time"), CurrentAnimPos);
						break;
					}
				}
			}
		}
	}
#endif
}

TArray<const FAnimNode_AssetPlayerBase*> FAnimInstanceProxy::GetInstanceAssetPlayers(const FName& GraphName) const
{
	TArray<const FAnimNode_AssetPlayerBase*> Nodes;

	// Retrieve all asset player nodes from the (named) Animation Layer Graph
	if (AnimClassInterface)
	{
		const TMap<FName, FGraphAssetPlayerInformation>& GrapInformationMap = AnimClassInterface->GetGraphAssetPlayerInformation();
		if (const FGraphAssetPlayerInformation* Information = GrapInformationMap.Find(GraphName))
		{
			for (const int32& NodeIndex : Information->PlayerNodeIndices)
			{
				if (const FAnimNode_AssetPlayerBase* Node = GetNodeFromIndex<FAnimNode_AssetPlayerBase>(NodeIndex))
				{
					Nodes.Add(Node);
				}
			}
		}
	}

	return Nodes;
}

TArray<FAnimNode_AssetPlayerBase*> FAnimInstanceProxy::GetMutableInstanceAssetPlayers(const FName& GraphName)
{
	TArray<FAnimNode_AssetPlayerBase*> Nodes;

	// Retrieve all asset player nodes from the (named) Animation Layer Graph	
	if (AnimClassInterface)
	{
		const TMap<FName, FGraphAssetPlayerInformation>& GrapInformationMap = AnimClassInterface->GetGraphAssetPlayerInformation();
		if (const FGraphAssetPlayerInformation* Information = GrapInformationMap.Find(GraphName))
		{
			for (const int32& NodeIndex : Information->PlayerNodeIndices)
			{
				if (FAnimNode_AssetPlayerBase* Node = GetMutableNodeFromIndex<FAnimNode_AssetPlayerBase>(NodeIndex))
				{
					Nodes.Add(Node);
				}
			}
		}
	}

	return Nodes;
}
TArray<const FAnimNode_AssetPlayerRelevancyBase*> FAnimInstanceProxy::GetInstanceRelevantAssetPlayers(const FName& GraphName) const
{
	TArray<const FAnimNode_AssetPlayerRelevancyBase*> Nodes;

	// Retrieve all asset player nodes from the (named) Animation Layer Graph
	if (AnimClassInterface)
	{
		const TMap<FName, FGraphAssetPlayerInformation>& GrapInformationMap = AnimClassInterface->GetGraphAssetPlayerInformation();
		if (const FGraphAssetPlayerInformation* Information = GrapInformationMap.Find(GraphName))
		{
			for (const int32& NodeIndex : Information->PlayerNodeIndices)
			{
				if (const FAnimNode_AssetPlayerRelevancyBase* Node = GetNodeFromIndex<FAnimNode_AssetPlayerRelevancyBase>(NodeIndex))
				{
					Nodes.Add(Node);
				}
			}
		}
	}

	return Nodes;
}

TArray<FAnimNode_AssetPlayerRelevancyBase*> FAnimInstanceProxy::GetMutableInstanceRelevantAssetPlayers(const FName& GraphName)
{
	TArray<FAnimNode_AssetPlayerRelevancyBase*> Nodes;

	// Retrieve all asset player nodes from the (named) Animation Layer Graph	
	if (AnimClassInterface)
	{
		const TMap<FName, FGraphAssetPlayerInformation>& GrapInformationMap = AnimClassInterface->GetGraphAssetPlayerInformation();
		if (const FGraphAssetPlayerInformation* Information = GrapInformationMap.Find(GraphName))
		{
			for (const int32& NodeIndex : Information->PlayerNodeIndices)
			{
				if (FAnimNode_AssetPlayerRelevancyBase* Node = GetMutableNodeFromIndex<FAnimNode_AssetPlayerRelevancyBase>(NodeIndex))
				{
					Nodes.Add(Node);
				}
			}
		}
	}

	return Nodes;
}

#if WITH_EDITOR
void FAnimInstanceProxy::RecordNodeVisit(int32 TargetNodeIndex, int32 SourceNodeIndex, float BlendWeight)
{
	UpdatedNodesThisFrame.Emplace(SourceNodeIndex, TargetNodeIndex, BlendWeight);
}

void FAnimInstanceProxy::RecordNodeAttribute(const FAnimInstanceProxy& InSourceProxy, int32 InTargetNodeIndex, int32 InSourceNodeIndex, FName InAttribute)
{
	TArray<FAnimBlueprintDebugData::FAttributeRecord>& InputAttributeRecords = NodeInputAttributesThisFrame.FindOrAdd(InTargetNodeIndex);
	InputAttributeRecords.Emplace(InSourceNodeIndex, InAttribute);

	if(&InSourceProxy == this)
	{
		TArray<FAnimBlueprintDebugData::FAttributeRecord>& OutputAttributeRecords = NodeOutputAttributesThisFrame.FindOrAdd(InSourceNodeIndex);
		OutputAttributeRecords.Emplace(InTargetNodeIndex, InAttribute);
	}
}

void FAnimInstanceProxy::RegisterWatchedPose(const FCompactPose& Pose, const FBlendedCurve& Curve, int32 LinkID)
{
	FPoseContext Context(this);
	Context.Pose = Pose;
	Context.Curve = Curve;

	RegisterWatchedPose(Context, LinkID);
}

void FAnimInstanceProxy::RegisterWatchedPose(const FCSPose<FCompactPose>& Pose, const FBlendedCurve& Curve, int32 LinkID)
{
	FComponentSpacePoseContext Context(this);
	Context.Pose = Pose;
	Context.Curve = Curve;

	RegisterWatchedPose(Context, LinkID);
}

void FAnimInstanceProxy::RegisterWatchedPose(const FPoseContext& PoseContext, int32 LinkID)
{
	if (bIsBeingDebugged)
	{
		FAnimBlueprintDebugData* DebugData = GetAnimBlueprintDebugData();
		if (DebugData)
		{
			if (USkeletalMeshComponent* SkelMeshComponent = GetSkelMeshComponent())
			{
				for (FAnimNodePoseWatch& PoseWatch : DebugData->AnimNodePoseWatch)
				{
					if (PoseWatch.PoseWatch && PoseWatch.NodeID == LinkID)
					{
						PoseWatch.Object = GetAnimInstanceObject();
						PoseWatch.PoseWatch->SetIsNodeEnabled(true);

						TArray<FTransform> BoneTransforms;
						BoneTransforms.AddUninitialized(PoseContext.Pose.GetBoneContainer().GetNumBones());

						TArray<FBoneIndexType> TmpRequiredBones;
						TmpRequiredBones.Reserve(PoseContext.Pose.GetBoneContainer().GetNumBones());

						for (FCompactPoseBoneIndex BoneIndex : PoseContext.Pose.ForEachBoneIndex())
						{
							FMeshPoseBoneIndex MeshBoneIndex = PoseContext.Pose.GetBoneContainer().MakeMeshPoseIndex(BoneIndex);
							BoneTransforms[MeshBoneIndex.GetInt()] = PoseContext.Pose[BoneIndex];
							TmpRequiredBones.Add(MeshBoneIndex.GetInt());
						}

						PoseWatch.SetPose(TmpRequiredBones, BoneTransforms);
						PoseWatch.SetCurves(PoseContext.Curve);
						PoseWatch.SetWorldTransform(SkelMeshComponent->GetComponentTransform());

						UE::Anim::FMeshAttributeContainer MeshAttributeContainer;
						MeshAttributeContainer.CopyFrom(PoseContext.CustomAttributes, PoseContext.Pose.GetBoneContainer());
						PoseWatch.SetAttributes(MeshAttributeContainer);

						TRACE_ANIM_POSE_WATCH(*this, PoseWatch.PoseWatchPoseElement, PoseWatch.NodeID, PoseWatch.GetBoneTransforms(), PoseWatch.GetCurves(), PoseWatch.GetRequiredBones(), PoseWatch.GetWorldTransform(), PoseWatch.PoseWatchPoseElement->GetIsVisible());
						break;
					}
				}
			}
		}
	}
}

void FAnimInstanceProxy::RegisterWatchedPose(const FComponentSpacePoseContext& PoseContext, int32 LinkID)
{
	if (bIsBeingDebugged)
	{
		FAnimBlueprintDebugData* DebugData = GetAnimBlueprintDebugData();
		if (DebugData)
		{
			if (USkeletalMeshComponent* SkelMeshComponent = GetSkelMeshComponent())
			{
				for (FAnimNodePoseWatch& PoseWatch : DebugData->AnimNodePoseWatch)
				{
					if (PoseWatch.PoseWatch && PoseWatch.NodeID == LinkID)
					{
						FCompactPose TempPose;
						FCSPose<FCompactPose>::ConvertComponentPosesToLocalPoses(PoseContext.Pose, TempPose);
						PoseWatch.Object = GetAnimInstanceObject();
						PoseWatch.PoseWatch->SetIsNodeEnabled(true);

						const TArray<FTransform, FAnimStackAllocator>& BoneTransforms = TempPose.GetBones();
						const TArray<FBoneIndexType>& TmpRequiredBones = TempPose.GetBoneContainer().GetBoneIndicesArray();
						PoseWatch.SetPose(TmpRequiredBones, BoneTransforms);
						PoseWatch.SetCurves(PoseContext.Curve);
						PoseWatch.SetWorldTransform(SkelMeshComponent->GetComponentTransform());

						UE::Anim::FMeshAttributeContainer MeshAttributeContainer;
						MeshAttributeContainer.CopyFrom(PoseContext.CustomAttributes, TempPose.GetBoneContainer());
						PoseWatch.SetAttributes(MeshAttributeContainer);

						TRACE_ANIM_POSE_WATCH(*this, PoseWatch.PoseWatchPoseElement, PoseWatch.NodeID, PoseWatch.GetBoneTransforms(), PoseWatch.GetCurves(), PoseWatch.GetRequiredBones(), PoseWatch.GetWorldTransform(), PoseWatch.PoseWatchPoseElement->GetIsVisible());
						break;
					}
				}
			}
		}
	}
}
#endif

FPoseSnapshot& FAnimInstanceProxy::AddPoseSnapshot(FName SnapshotName)
{
	FPoseSnapshot* PoseSnapshot = PoseSnapshots.FindByPredicate([SnapshotName](const FPoseSnapshot& PoseData) { return PoseData.SnapshotName == SnapshotName; });
	if (PoseSnapshot)
	{
		// Recycle an existing snapshot
		PoseSnapshot->Reset();
	}
	else
	{
		// Add a new empty snapshot
		PoseSnapshot = &PoseSnapshots[PoseSnapshots.AddDefaulted()];
	}
	PoseSnapshot->SnapshotName = SnapshotName;
	return *PoseSnapshot;
}

void FAnimInstanceProxy::RemovePoseSnapshot(FName SnapshotName)
{
	int32 Index = PoseSnapshots.IndexOfByPredicate([SnapshotName](const FPoseSnapshot& PoseData) { return PoseData.SnapshotName == SnapshotName; });
	if (Index != INDEX_NONE)
	{
		PoseSnapshots.RemoveAtSwap(Index);
	}
}

const FPoseSnapshot* FAnimInstanceProxy::GetPoseSnapshot(FName SnapshotName) const
{
	return PoseSnapshots.FindByPredicate([SnapshotName](const FPoseSnapshot& PoseData) { return PoseData.SnapshotName == SnapshotName; });
}

void FAnimInstanceProxy::ResetAnimationCurves()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	for (uint8 Index = 0; Index < (uint8)EAnimCurveType::MaxAnimCurveType; ++Index)
	{
		AnimationCurves[Index].Reset();
	}
}

void FAnimInstanceProxy::UpdateCurvesToEvaluationContext(const FAnimationEvaluationContext& InContext)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_UpdateCurvesToEvaluationContext);

	// Track material params we set last time round so we can clear them if they aren't set again.
	MaterialParametersToClear.Reset();
	for(auto Iter = AnimationCurves[(uint8)EAnimCurveType::MaterialCurve].CreateConstIterator(); Iter; ++Iter)
	{
		MaterialParametersToClear.Add(Iter.Key());
	}

	ResetAnimationCurves();

	InContext.Curve.ForEachElement([this](const UE::Anim::FCurveElement& InCurveElement)
	{
		AnimationCurves[(uint8)EAnimCurveType::AttributeCurve].Add(InCurveElement.Name, InCurveElement.Value);
	});

	UE::Anim::FNamedValueArrayUtils::Intersection(InContext.Curve, RequiredBones->GetCurveFlags(), 
		[this](const UE::Anim::FCurveElement& InCurveElement, const UE::Anim::FCurveElementFlags& InCurveFlagsElement)
		{
			if(EnumHasAnyFlags(InCurveFlagsElement.Flags | InCurveElement.Flags, UE::Anim::ECurveElementFlags::MorphTarget))
			{
				AnimationCurves[(uint8)EAnimCurveType::MorphTargetCurve].Add(InCurveElement.Name, InCurveElement.Value);
			}

			if(EnumHasAnyFlags(InCurveFlagsElement.Flags | InCurveElement.Flags, UE::Anim::ECurveElementFlags::Material))
			{
				const uint32 HashedName = GetTypeHash(InCurveElement.Name);
				MaterialParametersToClear.RemoveByHash(HashedName, InCurveElement.Name);
				AnimationCurves[(uint8)EAnimCurveType::MaterialCurve].AddByHash(HashedName, InCurveElement.Name, InCurveElement.Value);
			}
		});
}

void FAnimInstanceProxy::UpdateCurvesPostEvaluation(USkeletalMeshComponent* SkelMeshComp)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_UpdateCurvesPostEvaluation);

	// Add curves to reset parameters that we have previously set but didn't tick this frame.
	for(FName MaterialParameterToClear : MaterialParametersToClear)
	{
		// when reset, we go back to default value
		float DefaultValue = SkelMeshComp->GetScalarParameterDefaultValue(MaterialParameterToClear);
		AnimationCurves[(uint8)EAnimCurveType::MaterialCurve].Add(MaterialParameterToClear, DefaultValue);
	}

	// update curves to component
	SkelMeshComp->ApplyAnimationCurvesToComponent(&AnimationCurves[(uint8)EAnimCurveType::MaterialCurve], &AnimationCurves[(uint8)EAnimCurveType::MorphTargetCurve]);

	// Remove cleared params now they have been pushed to the mesh
	for(FName MaterialParameterToClear : MaterialParametersToClear)
	{
		AnimationCurves[(uint8)EAnimCurveType::MaterialCurve].Remove(MaterialParameterToClear);
	}
}

bool FAnimInstanceProxy::HasActiveCurves() const
{
	for(const TMap<FName, float>& AnimationCurveMap : AnimationCurves)
	{
		if(AnimationCurveMap.Num() > 0)
		{
			return true;
		}
	}
	return false;
}

void FAnimInstanceProxy::AddCurveValue(const FName& CurveName, float Value, bool bMorphtarget, bool bMaterial)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// save curve value, it will overwrite if same exists, 
	//CurveValues.Add(CurveName, Value);
	float* CurveValPtr = AnimationCurves[(uint8)EAnimCurveType::AttributeCurve].Find(CurveName);
	if ( CurveValPtr )
	{
		// sum up, in the future we might normalize, but for now this just sums up
		// this won't work well if all of them have full weight - i.e. additive 
		*CurveValPtr = Value;
	}
	else
	{
		AnimationCurves[(uint8)EAnimCurveType::AttributeCurve].Add(CurveName, Value);
	}

	if (bMorphtarget)
	{
		CurveValPtr = AnimationCurves[(uint8)EAnimCurveType::MorphTargetCurve].Find(CurveName);
		if (CurveValPtr)
		{
			// sum up, in the future we might normalize, but for now this just sums up
			// this won't work well if all of them have full weight - i.e. additive 
			*CurveValPtr = Value;
		}
		else
		{
			AnimationCurves[(uint8)EAnimCurveType::MorphTargetCurve].Add(CurveName, Value);
		}
	}
	if (bMaterial)
	{
		MaterialParametersToClear.Remove(CurveName);
		CurveValPtr = AnimationCurves[(uint8)EAnimCurveType::MaterialCurve].Find(CurveName);
		if (CurveValPtr)
		{
			*CurveValPtr = Value;
		}
		else
		{
			AnimationCurves[(uint8)EAnimCurveType::MaterialCurve].Add(CurveName, Value);
		}
	}
}

FAnimBlueprintDebugData* FAnimInstanceProxy::GetAnimBlueprintDebugData() const
{
#if WITH_EDITORONLY_DATA
	if (bIsBeingDebugged)
	{
		UAnimBlueprint* AnimBP = GetAnimBlueprint();
		return AnimBP ? AnimBP->GetDebugData() : nullptr;
	}
#endif

	return nullptr;
}

void FAnimInstanceProxy::InitializeInputProxy(FAnimInstanceProxy* InputProxy, UAnimInstance* InAnimInstance)
{
	if (InAnimInstance && InputProxy)
	{
		InputProxy->Initialize(InAnimInstance);
	}
}

void FAnimInstanceProxy::GatherInputProxyDebugData(FAnimInstanceProxy* InputProxy, FNodeDebugData& DebugData)
{
	if (InputProxy)
	{
		InputProxy->GatherDebugData(DebugData);
	}
}

void FAnimInstanceProxy::CacheBonesInputProxy(FAnimInstanceProxy* InputProxy)
{
	if (InputProxy)
	{
		InputProxy->CacheBones();
	}
}

void FAnimInstanceProxy::UpdateInputProxy(FAnimInstanceProxy* InputProxy, const FAnimationUpdateContext& Context)
{
	if (InputProxy)
	{
		InputProxy->UpdateAnimationNode(Context);
	}
}

void FAnimInstanceProxy::EvaluateInputProxy(FAnimInstanceProxy* InputProxy, FPoseContext& Output)
{
	if (InputProxy)
	{
		if(!InputProxy->Evaluate(Output))
		{
			Output.ResetToRefPose();
		}
	}
}

void FAnimInstanceProxy::ResetCounterInputProxy(FAnimInstanceProxy* InputProxy)
{
	if (InputProxy)
	{
		InputProxy->UpdateCounter.Reset();
		InputProxy->EvaluationCounter.Reset();
		InputProxy->CachedBonesCounter.Reset();
	}
}

void FAnimInstanceProxy::ForEachStateMachine(const TFunctionRef<void(FAnimNode_StateMachine&)>& Functor)
{
	if (AnimClassInterface)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
		for (int32 MachineIndex = 0; MachineIndex < AnimNodeProperties.Num(); MachineIndex++)
		{
			FStructProperty* Property = AnimNodeProperties[AnimNodeProperties.Num() - 1 - MachineIndex];
			if (Property && Property->Struct->IsChildOf(FAnimNode_StateMachine::StaticStruct()))
			{
				FAnimNode_StateMachine* StateMachine = Property->ContainerPtrToValuePtr<FAnimNode_StateMachine>(AnimInstanceObject);
				if (StateMachine)
				{
					Functor(*StateMachine);
				}
			}
		}
	}
}

void FAnimInstanceProxy::SetInterpolationOverride(TOptional<EAnimInterpolationType> InterpolationType)
{
	InterpolationOverride = InterpolationType;
}

TOptional<EAnimInterpolationType> FAnimInstanceProxy::GetInterpolationOverride() const
{
	return InterpolationOverride;
}

#undef LOCTEXT_NAMESPACE

