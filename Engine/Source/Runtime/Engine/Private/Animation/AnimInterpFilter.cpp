// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimInterpFilter.h"
#include "Engine/EngineTypes.h"

//======================================================================================================================
// FFIRFilter
//======================================================================================================================
float FFIRFilter::GetInterpolationCoefficient (EFilterInterpolationType InterpolationType, int32 CoefficientIndex) const
{
	const float Step = GetStep();

	switch (InterpolationType)
	{
		case BSIT_Average:
			return Step;	
		case BSIT_Linear:
			return Step*CoefficientIndex;
		case BSIT_Cubic:
			return Step*Step*Step*CoefficientIndex;
		default:
			// Note that BSIT_EaseInOut is not supported
			return 0.f;
	}
}

void FFIRFilter::CalculateCoefficient(EFilterInterpolationType InterpolationType)
{
	if ( IsValid() )
	{
		float Sum=0.f;
		for (int32 I=0; I != Coefficients.Num(); ++I)
		{
			Coefficients[I] = GetInterpolationCoefficient(InterpolationType, I);
			Sum += Coefficients[I];
		}

		// now normalize it, if not 1
		if ( fabs(Sum-1.f) > ZERO_ANIMWEIGHT_THRESH )
		{
			for (int32 I=0; I != Coefficients.Num(); ++I)
			{
				Coefficients[I]/=Sum;
			}
		}
	}
}

float FFIRFilter::GetFilteredData(float Input)
{
	if ( IsValid() )
	{
		FilterData[CurrentStack] = Input;
		float Result = CalculateFilteredOutput();
		if (++CurrentStack > FilterData.Num() - 1)
		{
			CurrentStack = 0;
		}

		LastOutput = Result;
		return Result;
	}

	LastOutput = Input;
	return Input;
}

float FFIRFilter::CalculateFilteredOutput() const
{
	float Output = 0.f;
	int32 StackIndex = CurrentStack;

	for ( int32 I=Coefficients.Num()-1; I>=0; --I )
	{
		Output += FilterData[StackIndex]*Coefficients[I];
		if (--StackIndex < 0)
		{
			StackIndex = FilterData.Num() - 1;
		}
	}

	return Output;
}

//======================================================================================================================
// FFIRFilterTimeBased
//======================================================================================================================
int32 FFIRFilterTimeBased::GetSafeCurrentStackIndex()
{
	// if valid range
	check ( CurrentStackIndex < FilterData.Num() );

	// see if it's expired yet
	if ( !FilterData[CurrentStackIndex].IsValid() )
	{
		return CurrentStackIndex;
	}

	// else see any other index is available
	// when you do this, go to forward, (oldest)
	// this should not be the case because most of times
	// current one should be the oldest one, but since 
	// we jumps when reallocation happens, we still do this
	for (int32 I=0; I != FilterData.Num(); ++I)
	{
		int32 NewIndex = CurrentStackIndex + I;
		if (NewIndex >= FilterData.Num())
		{
			NewIndex = NewIndex - FilterData.Num();
		}

		if ( !FilterData[NewIndex].IsValid() )
		{
			return NewIndex;
		}
	}

	// if current one isn't available anymore 
	// that means we need more stack
	const int32 NewIndex = FilterData.Num();
	FilterData.AddZeroed(5);
	return NewIndex;
}

void FFIRFilterTimeBased::RefreshValidFilters()
{
	if (FilterData.IsEmpty())
	{
		FilterData.Empty(10);
		FilterData.AddZeroed(10);
		CurrentStackIndex = 0;
	}
	else
	{
		// Ensure the current time is valid
		for (int32 I=0; I != FilterData.Num(); ++I)
		{
			FilterData[I].EnsureTimeIsValid(CurrentTime, WindowDuration);
		}
	}
}

void FFIRFilterTimeBased::WrapToValue(float Input, float Range)
{
	if (Range <= 0.0f)
	{
		return;
	}
	float HalfRange = Range / 2.0f;

	switch (InterpolationType)
	{
	case BSIT_ExponentialDecay:
	case BSIT_SpringDamper:
	{
		if (FilterData.Num() != 0)
		{
			FilterData[0].Input = FMath::Wrap(FilterData[0].Input, Input - HalfRange, Input + HalfRange);
		}
	}
	break;
	default:
	{
		if (IsValid())
		{
			float NewLastOutput = FMath::Wrap(LastOutput, Input - HalfRange, Input + HalfRange);
			float Delta = NewLastOutput - LastOutput;
			if (Delta)
			{
				LastOutput = NewLastOutput;
				for (int32 Index = 0; Index != FilterData.Num() ; ++Index)
				{
					FilterData[Index].Input += Delta;
				}
			}
		}
	}
	break;
	}
}

void FFIRFilterTimeBased::SetToValue(float Value)
{
	LastOutput = Value;

	if (FilterData.Num() > 0)
	{
		FilterData[0].Input = Value;
	}
}

float FFIRFilterTimeBased::UpdateAndGetFilteredData(float Input, float DeltaTime)
{
	// Early return if there is no smoothing - applies to all smoothing types. Note that if
	// WindowDuration changes we will have been re-initialized, so we don't need to worry about
	// updating the filter in this case.
	if (WindowDuration <= 0.0f)
	{
		LastOutput = Input;
		return Input;
	}

	if (DeltaTime > UE_KINDA_SMALL_NUMBER)
	{
		float Result;
		CurrentTime += DeltaTime;

		switch (InterpolationType)
		{
			case BSIT_ExponentialDecay:
			{
				if (FilterData.Num() != 1)
				{
					FilterData.Empty(1);
					FilterData.Push(FFilterData(Input, 0.0f));
				}
				const float OrigValue = FilterData[0].Input;
				FMath::ExponentialSmoothingApprox(FilterData[0].Input, Input, DeltaTime, WindowDuration / UE_EULERS_NUMBER);
				if (MaxSpeed > 0.0f)
				{
					// Clamp the speed
					FilterData[0].Input = FMath::Clamp(FilterData[0].Input, OrigValue - MaxSpeed * DeltaTime,
                                                         OrigValue + MaxSpeed * DeltaTime); 
				}
				Result = FilterData[0].Input;
			}
			break;
			case BSIT_SpringDamper:
			{
				if (FilterData.Num() != 2)
				{
					FilterData.Empty(2);
					// [0] element is the value, [1] element is the rate
					FilterData.Push(FFilterData(Input, 0.0f));
					FilterData.Push(FFilterData(0.0f, 0.0f));
				}
				const float OrigValue = FilterData[0].Input;
				FMath::SpringDamperSmoothing(FilterData[0].Input, FilterData[1].Input, Input, 0.0f, DeltaTime,
				                             WindowDuration / UE_EULERS_NUMBER, DampingRatio);
				if (MaxSpeed > 0.0f)
				{
					// Clamp the speed
					FilterData[0].Input = FMath::Clamp(FilterData[0].Input, OrigValue - MaxSpeed * DeltaTime,
                                                         OrigValue + MaxSpeed * DeltaTime); 
					FilterData[1].Input = FMath::Clamp(FilterData[1].Input, -MaxSpeed, MaxSpeed);
				}
				if (bClamp)
				{
					// Clamp the value
					if (FilterData[0].Input > MaxValue)
					{
						FilterData[0].Input = MaxValue;
						if (FilterData[1].Input > 0.0f)
						{
							FilterData[1].Input = 0.0f;
						}
					}
					if (FilterData[0].Input < MinValue)
					{
						FilterData[0].Input = MinValue;
						if (FilterData[1].Input < 0.0f)
						{
							FilterData[1].Input = 0.0f;
						}
					}
				}
				Result = FilterData[0].Input;
			}
			break;
			default:
			{
				// This handles the array based filters
				if (IsValid())
				{
					RefreshValidFilters();

					CurrentStackIndex = GetSafeCurrentStackIndex();
					FilterData[CurrentStackIndex].SetInput(Input, CurrentTime);
					Result = CalculateFilteredOutput();
					if (++CurrentStackIndex > FilterData.Num() - 1)
					{
						CurrentStackIndex = 0;
					}
				}
				else
				{
					Result = Input;
				}
			}
			break;
		}
		LastOutput = Result;
		return Result;
	}

	return LastOutput;
}

float FFIRFilterTimeBased::GetInterpolationCoefficient(const FFilterData& Data) const
{
	if (Data.IsValid())
	{
		const float Diff = Data.Diff(CurrentTime);
		if (Diff<=WindowDuration)
		{
			switch(InterpolationType)
			{
			case BSIT_Average:
				return 1.f;
			case BSIT_Linear:
				return 1.f - Diff/WindowDuration;
			case BSIT_Cubic:
				return 1.f - FMath::Cube(Diff/WindowDuration);
			case BSIT_EaseInOut:
				// Quadratic that starts and ends at 0, and reaches 1 half way through the window
				return 1.0f - 4.0f * FMath::Square(Diff/WindowDuration - 0.5f);
			default:
				break;
			}
		}
	}

	return 0.f;
}

float FFIRFilterTimeBased::CalculateFilteredOutput()
{
	check ( IsValid() );
	float SumCoefficient = 0.f;
	float SumInputs = 0.f;
	for (int32 I=0; I != FilterData.Num(); ++I)
	{
		const float Coefficient = GetInterpolationCoefficient(FilterData[I]);
		if (Coefficient > 0.f)
		{
			SumCoefficient += Coefficient;
			SumInputs += Coefficient * FilterData[I].Input;
		}
	}
	return SumCoefficient > 0.f ? SumInputs/SumCoefficient : 0.f;
}

