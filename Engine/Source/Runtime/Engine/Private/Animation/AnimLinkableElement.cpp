// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimLinkableElement.h"
#include "Animation/AnimMontage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimLinkableElement)

void FAnimLinkableElement::Clear()
{
	ChangeLinkMethod(EAnimLinkMethod::Absolute);
	LinkedSequence = nullptr;
	SegmentBeginTime = -1.0f;
	SegmentLength = -1.0f;
	SegmentIndex = INDEX_NONE;
}

void FAnimLinkableElement::Update()
{
	if(LinkedMontage && LinkedMontage->SlotAnimTracks.IsValidIndex(SlotIndex))
	{
		FSlotAnimationTrack& Slot = LinkedMontage->SlotAnimTracks[SlotIndex];
		float CurrentTime = GetTime();

		// If we don't have a segment, check to see if one has been added.
		if(SegmentIndex == INDEX_NONE || !Slot.AnimTrack.AnimSegments.IsValidIndex(SegmentIndex))
		{
			SegmentIndex = Slot.AnimTrack.GetSegmentIndexAtTime(CurrentTime);
		}

		if(SegmentIndex != INDEX_NONE)
		{
			// Update timing info from current segment
			FAnimSegment& Segment = Slot.AnimTrack.AnimSegments[SegmentIndex];
			LinkedSequence = Segment.GetAnimReference();
			SegmentBeginTime = Segment.StartPos;
			SegmentLength = Segment.GetLength();

			// Handle Relative link mode, make sure to stay within the linked segment
			if(CachedLinkMethod == EAnimLinkMethod::Relative)
			{
				float SegmentEnd = SegmentBeginTime + SegmentLength;
				if(GetTime() > SegmentEnd)
				{
					SetTime(SegmentEnd);
				}
			}

			// Relink if necessary
			ConditionalRelink();
		}
	}
}

void FAnimLinkableElement::OnChanged(float NewMontageTime)
{
	// Only update linkage in a montage.
	if(!LinkedMontage)
	{
		return;
	}

	SlotIndex = FMath::Clamp(SlotIndex, 0, LinkedMontage->SlotAnimTracks.Num()-1);

	// If the link method changed, transform the link value
	if(CachedLinkMethod != LinkMethod)
	{
		float AbsTime = -1.0f;
		switch(CachedLinkMethod)
		{
			case EAnimLinkMethod::Absolute:
				AbsTime = LinkValue;
				break;
			case EAnimLinkMethod::Relative:
				AbsTime = GetTimeFromRelative(EAnimLinkMethod::Absolute);
				break;
			case EAnimLinkMethod::Proportional:
				AbsTime = GetTimeFromProportional(EAnimLinkMethod::Absolute);
				break;
		}
		check(AbsTime != -1.0f);
		CachedLinkMethod = LinkMethod;

		// We aren't changing the time, just transforming it so use internal settime
		SetTime_Internal(AbsTime);
		NewMontageTime = AbsTime;
	}

	FSlotAnimationTrack& Slot = LinkedMontage->SlotAnimTracks[SlotIndex];

	SegmentIndex = Slot.AnimTrack.GetSegmentIndexAtTime(NewMontageTime);
	if(SegmentIndex != INDEX_NONE)
	{
		// Update to the detected segment
		FAnimSegment& Segment = Slot.AnimTrack.AnimSegments[SegmentIndex];
		LinkedSequence = Segment.GetAnimReference();
		SegmentBeginTime = Segment.StartPos;
		SegmentLength = Segment.GetLength();

		SetTime(NewMontageTime);
	}
	else if(!LinkedSequence)
	{
		// We have no segment to link to, we need to clear our the segment data
		// and give ourselves an absolute time
		LinkValue = NewMontageTime;
		Clear();
	}
}

FAnimSegment* FAnimLinkableElement::GetSegmentAtCurrentTime()
{
	FAnimSegment* Result = nullptr;
	if(LinkedMontage)
	{
		Result = LinkedMontage->SlotAnimTracks[SlotIndex].AnimTrack.GetSegmentAtTime(GetTime());
	}
	return Result;
}

float FAnimLinkableElement::GetTime(EAnimLinkMethod::Type ReferenceFrame /*= EMontageLinkMethod::Absolute*/) const
{
	if(ReferenceFrame != CachedLinkMethod)
	{
		switch(CachedLinkMethod)
		{
			case EAnimLinkMethod::Absolute:
			{
				return GetTimeFromAbsolute(ReferenceFrame);
			}
			case EAnimLinkMethod::Relative:
			{
				return GetTimeFromRelative(ReferenceFrame);
			}
			case EAnimLinkMethod::Proportional:
			{
				return GetTimeFromProportional(ReferenceFrame);
			}
		}
	}
	return LinkValue;
}

void FAnimLinkableElement::SetTime(float NewTime, EAnimLinkMethod::Type ReferenceFrame /*= EMontageLinkMethod::Absolute*/)
{
	SetTime_Internal(NewTime, ReferenceFrame);
}

float FAnimLinkableElement::GetTimeFromAbsolute(EAnimLinkMethod::Type ReferenceFrame) const
{
	switch(ReferenceFrame)
	{
		case EAnimLinkMethod::Relative:
		{
			return LinkValue - SegmentBeginTime;
		}
		case EAnimLinkMethod::Proportional:
		{
			return (LinkValue - SegmentBeginTime) / SegmentLength;
		}
	}
	return -1.0f;
}

float FAnimLinkableElement::GetTimeFromRelative(EAnimLinkMethod::Type ReferenceFrame) const
{
	switch(ReferenceFrame)
	{
		case EAnimLinkMethod::Absolute:
		{
			return SegmentBeginTime + LinkValue;
		}
		case EAnimLinkMethod::Proportional:
		{
			return LinkValue / SegmentLength;
		}
	}
	return -1.0f;
}

float FAnimLinkableElement::GetTimeFromProportional(EAnimLinkMethod::Type ReferenceFrame) const
{
	switch(ReferenceFrame)
	{
		case EAnimLinkMethod::Absolute:
		{
			return SegmentBeginTime + LinkValue * SegmentLength;
		}
		case EAnimLinkMethod::Relative:
		{
			return LinkValue * SegmentLength;
		}
	}
	return -1.0f;
}

void FAnimLinkableElement::SetTimeFromAbsolute(float NewTime, EAnimLinkMethod::Type ReferenceFrame)
{
	switch(ReferenceFrame)
	{
		case EAnimLinkMethod::Relative:
		{
			LinkValue = SegmentBeginTime + NewTime;
		}
		case EAnimLinkMethod::Proportional:
		{
			LinkValue = SegmentBeginTime + SegmentLength * NewTime;
		}
	}
}

void FAnimLinkableElement::SetTimeFromRelative(float NewTime, EAnimLinkMethod::Type ReferenceFrame)
{
	switch(ReferenceFrame)
	{
		case EAnimLinkMethod::Absolute:
		{
			LinkValue = NewTime - SegmentBeginTime;
		}
			break;
		case EAnimLinkMethod::Proportional:
		{
			LinkValue = NewTime * SegmentLength;
		}
			break;
	}
}

void FAnimLinkableElement::SetTimeFromProportional(float NewTime, EAnimLinkMethod::Type ReferenceFrame)
{
	switch(ReferenceFrame)
	{
		case EAnimLinkMethod::Absolute:
		{
			if(SegmentLength != 0.0f)
			{
				LinkValue = (NewTime - SegmentBeginTime) / SegmentLength;
			}
			else
			{
				// if segment length is 0, we can't set anywhere else but 0.f
				LinkValue = 0.f;
			}
		}
			break;
		case EAnimLinkMethod::Relative:
		{
			if(SegmentLength != 0.0f)
			{
				LinkValue = NewTime / SegmentLength;
			}
			else
			{
				// if segment length is 0, we can't set anywhere else but 0.f
				LinkValue = 0.f;			
			}
		}
			break;
	}
}

void FAnimLinkableElement::ChangeLinkMethod(EAnimLinkMethod::Type NewLinkMethod)
{
	if(NewLinkMethod != LinkMethod)
	{
		// Switch to the new link method and resolve it
		LinkMethod = NewLinkMethod;
		OnChanged(GetTime());
	}
}

void FAnimLinkableElement::ChangeSlotIndex(int32 NewSlotIndex)
{
	if(LinkedMontage)
	{
		Link(LinkedMontage, GetTime(), NewSlotIndex);
	}
}

void FAnimLinkableElement::SetTime_Internal(float NewTime, EAnimLinkMethod::Type ReferenceFrame)
{
	if(ReferenceFrame != CachedLinkMethod)
	{
		switch(CachedLinkMethod)
		{
			case EAnimLinkMethod::Absolute:
			{
				SetTimeFromAbsolute(NewTime, ReferenceFrame);
			}
				break;
			case EAnimLinkMethod::Relative:
			{
				SetTimeFromRelative(NewTime, ReferenceFrame);
			}
				break;
			case EAnimLinkMethod::Proportional:
			{
				SetTimeFromProportional(NewTime, ReferenceFrame);
			}
				break;
		}
	}
	else
	{
		LinkValue = NewTime;
	}
}

bool FAnimLinkableElement::ConditionalRelink()
{
	// Check slot index if we're in a montage
	bool bRequiresRelink = false;
	
	if(LinkedMontage)
	{
		if(!LinkedMontage->SlotAnimTracks.IsValidIndex(SlotIndex))
		{
			bRequiresRelink = true;
			SlotIndex = 0;
		}
	}

	// Check to see if we've moved to a new segment
	float CurrentAbsTime = GetTime();
	if(CurrentAbsTime < SegmentBeginTime || CurrentAbsTime > SegmentBeginTime + SegmentLength)
	{
		bRequiresRelink = true;
	}

	if(bRequiresRelink)
	{
		if(LinkedMontage)
		{
			Link(LinkedMontage, CurrentAbsTime, SlotIndex);
		}
		else if(LinkedSequence)
		{
			Link(LinkedSequence, CurrentAbsTime);
		}
	}

	return bRequiresRelink;
}

void FAnimLinkableElement::Link(UAnimSequenceBase* AnimSequenceBase, float AbsTime, int32 InSlotIndex /*= 0*/)
{
	if(UAnimMontage* Montage = Cast<UAnimMontage>(AnimSequenceBase))
	{
		if(Montage->SlotAnimTracks.Num() > 0)
		{
			LinkedMontage = Montage;

			SlotIndex = InSlotIndex;
			FSlotAnimationTrack& Slot = Montage->SlotAnimTracks[SlotIndex];

			SegmentIndex = Slot.AnimTrack.GetSegmentIndexAtTime(AbsTime);
			if(SegmentIndex != INDEX_NONE)
			{
				const FAnimSegment& Segment = Slot.AnimTrack.AnimSegments[SegmentIndex];
				LinkedSequence = Segment.GetAnimReference();
				SegmentBeginTime = Segment.StartPos;
				SegmentLength = Segment.GetLength();

				SetTime_Internal(AbsTime);
			}
			else
			{
				// Nothing to link to
				// We have no segment to link to, we need to clear our the segment data
				// and give ourselves an absolute time
				LinkValue = AbsTime;
				LinkedSequence = nullptr;
				SegmentBeginTime = -1.0f;
				SegmentLength = -1.0f;
				LinkMethod = EAnimLinkMethod::Absolute;
				CachedLinkMethod = LinkMethod;
			}
		}
	}
	else if (AnimSequenceBase && AnimSequenceBase->GetPlayLength() > 0)
	{		
		LinkedMontage = nullptr;
		LinkedSequence = AnimSequenceBase;
		SegmentIndex = 0;

		SegmentBeginTime = 0.0f;
		SegmentLength = AnimSequenceBase->GetPlayLength();

		SetTime(AbsTime);
	}
}

void FAnimLinkableElement::RefreshSegmentOnLoad()
{
	// We only perform this step if we have valid data from a previous link
	if(LinkedMontage && SegmentIndex != INDEX_NONE && SlotIndex != INDEX_NONE)
	{
		if(LinkedMontage->SlotAnimTracks.IsValidIndex(SlotIndex))
		{
			FSlotAnimationTrack& Slot = LinkedMontage->SlotAnimTracks[SlotIndex];
			if(Slot.AnimTrack.AnimSegments.IsValidIndex(SegmentIndex))
			{
				FAnimSegment& Segment = Slot.AnimTrack.AnimSegments[SegmentIndex];

				if(Segment.GetAnimReference() == LinkedSequence)
				{
					if(CachedLinkMethod == EAnimLinkMethod::Relative)
					{
						LinkValue = FMath::Clamp<float>(LinkValue, 0.0f, Segment.GetLength());
					}

					// Update timing
					SegmentBeginTime = Segment.StartPos;
					SegmentLength = Segment.GetLength();
				}
			}
		}
	}
}

