// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimMetaData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimMetaData)

/////////////////////////////////////////////////////
// UAnimMetaDatay

UAnimMetaData::UAnimMetaData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

