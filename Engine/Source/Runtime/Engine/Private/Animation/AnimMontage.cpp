// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimMontage.cpp: Montage classes that contains slots
=============================================================================*/ 

#include "Animation/AnimMontage.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "UObject/LinkerLoad.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "UObject/ObjectSaveContext.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "UObject/UObjectThreadContext.h"
#include "Animation/AssetMappingTable.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendProfile.h"
#include "AnimationUtils.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimTrace.h"
#include "Animation/ActiveMontageInstanceScope.h"
#include "Animation/AnimationSettings.h"
#include "Logging/StructuredLog.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimMontage)

DEFINE_LOG_CATEGORY(LogAnimMontage);

DECLARE_CYCLE_STAT(TEXT("AnimMontageInstance_Advance"), STAT_AnimMontageInstance_Advance, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("AnimMontageInstance_TickBranchPoints"), STAT_AnimMontageInstance_TickBranchPoints, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("AnimMontageInstance_Advance_Iteration"), STAT_AnimMontageInstance_Advance_Iteration, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("AnimMontageInstance_Terminate"), STAT_AnimMontageInstance_Terminate, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("AnimMontageInstance_HandleEvents"), STAT_AnimMontageInstance_HandleEvents, STATGROUP_Anim);

// Pre-built FNames so we don't take the hit of constructing FNames at spawn time
namespace MontageFNames
{
	static FName TimeStretchCurveName("MontageTimeStretchCurve");
}

// CVars
namespace MontageCVars
{
	static bool bEndSectionRequiresTimeRemaining = false;
	static FAutoConsoleVariableRef CVarMontageEndSectionRequiresTimeRemaining(
		TEXT("a.Montage.EndSectionRequiresTimeRemaining"),
		bEndSectionRequiresTimeRemaining,
		TEXT("Montage EndOfSection is only checked if there is remaining time (default false)."));
	extern bool bEarlyOutMontageWhenUninitialized;
	
	static bool bUseForcedFromPositionAsSubStepOrigin = true;
	static FAutoConsoleVariableRef CVarUseForcedFromPositionAsSubStepOrigin(
		TEXT("a.Montage.UseForcedFromPositionAsSubStepOrigin"),
		bUseForcedFromPositionAsSubStepOrigin,
		TEXT(" When true (default), if a ForcedNextFromPosition is set during Montage Advance, use that forced position as the sub-step start point rather than the stale cached Position."));
	
	static bool bUseForcedSequencerMontageLoopBehavior = false;
	static FAutoConsoleVariableRef CVarUseForcedSequencerMontageLoopBehavior(
		TEXT("a.Montage.UseForcedSequencerMontageLoopBehavior"),
		bUseForcedSequencerMontageLoopBehavior,
		TEXT(" When true, if a montage is updated from Sequencer by utilising forced positions also handle wrapping correctly when the Sequencer loops"));
	
} // end namespace MontageCVars

///////////////////////////////////////////////////////////////////////////
//
UAnimMontage::UAnimMontage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BlendModeIn = EMontageBlendMode::Standard;
	BlendModeOut = EMontageBlendMode::Standard;

	BlendIn.SetBlendTime(0.25f);
	BlendOut.SetBlendTime(0.25f);
	BlendOutTriggerTime = -1.f;
	bEnableAutoBlendOut = true;
	SyncSlotIndex = 0;

	BlendProfileIn = nullptr;
	BlendProfileOut = nullptr;

#if WITH_EDITORONLY_DATA
	BlendInTime_DEPRECATED = -1.f;
	BlendOutTime_DEPRECATED = -1.f;
#endif

	AddSlot(FAnimSlotGroup::DefaultSlotName);

	TimeStretchCurveName = MontageFNames::TimeStretchCurveName;
}

FSlotAnimationTrack& UAnimMontage::AddSlot(FName SlotName)
{
	int32 NewSlot = SlotAnimTracks.AddDefaulted(1);
	SlotAnimTracks[NewSlot].SlotName = SlotName;
	return SlotAnimTracks[NewSlot];
}

bool UAnimMontage::IsValidSlot(FName InSlotName) const
{
	for (int32 I=0; I<SlotAnimTracks.Num(); ++I)
	{
		if ( SlotAnimTracks[I].SlotName == InSlotName )
		{
			// if data is there, return true. Otherwise, it doesn't matter
			return ( SlotAnimTracks[I].AnimTrack.AnimSegments.Num() >  0 );
		}
	}

	return false;
}

bool UAnimMontage::IsDynamicMontage() const
{
	return GetPackage() == GetTransientPackage();
}

UAnimSequenceBase* UAnimMontage::GetFirstAnimReference() const
{
	if(!SlotAnimTracks.IsEmpty() && !SlotAnimTracks[0].AnimTrack.AnimSegments.IsEmpty())
	{
		return SlotAnimTracks[0].AnimTrack.AnimSegments[0].GetAnimReference();
	}

	return nullptr;
}

#if WITH_EDITORONLY_DATA
void UAnimMontage::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Note this happens before serialization
			BlendIn.SetBlendOption(EAlphaBlendOption::Linear);
			BlendOut.SetBlendOption(EAlphaBlendOption::Linear);
		}
	}
	
	Super::Serialize(Ar);
}
#endif // WITH_EDITORONLY_DATA

const FAnimTrack* UAnimMontage::GetAnimationData(FName InSlotName) const
{
	for (int32 I=0; I<SlotAnimTracks.Num(); ++I)
	{
		if ( SlotAnimTracks[I].SlotName == InSlotName )
		{
			// if data is there, return true. Otherwise, it doesn't matter
			return &( SlotAnimTracks[I].AnimTrack );
		}
	}

	return nullptr;
}

bool UAnimMontage::IsWithinPos(int32 FirstIndex, int32 SecondIndex, float CurrentTime) const
{
	float StartTime;
	float EndTime;
	if ( CompositeSections.IsValidIndex(FirstIndex) )
	{
		StartTime = CompositeSections[FirstIndex].GetTime();
	}
	else // if first index isn't valid, set to be 0.f, so it starts from reset
	{
		StartTime = 0.f;
	}

	if ( CompositeSections.IsValidIndex(SecondIndex) )
	{
		EndTime = CompositeSections[SecondIndex].GetTime();
	}
	else // if end index isn't valid, set to be BIG_NUMBER
	{
		// @todo anim, I don't know if using SequenceLength is better or BIG_NUMBER
		// I don't think that'd matter. 
		EndTime = GetPlayLength();
	}

	// since we do range of [StartTime, EndTime) (excluding EndTime) 
	// there is blindspot of when CurrentTime becomes >= SequenceLength
	// include that frame if CurrentTime gets there. 
	// Otherwise, we continue to use [StartTime, EndTime)
	if (CurrentTime >= GetPlayLength())
	{
		return (StartTime <= CurrentTime && EndTime >= CurrentTime);
	}

	return (StartTime <= CurrentTime && EndTime > CurrentTime);
}

float UAnimMontage::CalculatePos(FCompositeSection &Section, float PosWithinCompositeSection) const
{
	float Offset = Section.GetTime();
	Offset += PosWithinCompositeSection;
	// @todo anim
	return Offset;
}

int32 UAnimMontage::GetSectionIndexFromPosition(float Position) const
{
	for (int32 I=0; I<CompositeSections.Num(); ++I)
	{
		// if within
		if( IsWithinPos(I, I+1, Position) )
		{
			return I;
		}
	}

	return INDEX_NONE;
}

int32 UAnimMontage::GetAnimCompositeSectionIndexFromPos(float CurrentTime, float& PosWithinCompositeSection) const
{
	PosWithinCompositeSection = 0.f;

	for (int32 I=0; I<CompositeSections.Num(); ++I)
	{
		// if within
		if (IsWithinPos(I, I+1, CurrentTime))
		{
			PosWithinCompositeSection = CurrentTime - CompositeSections[I].GetTime();
			return I;
		}
	}

	return INDEX_NONE;
}

float UAnimMontage::GetSectionTimeLeftFromPos(float Position)
{
	const int32 SectionID = GetSectionIndexFromPosition(Position);
	if( SectionID != INDEX_NONE )
	{
		if( IsValidSectionIndex(SectionID+1) )
		{
			return (GetAnimCompositeSection(SectionID+1).GetTime() - Position);
		}
		else
		{
			return (GetPlayLength() - Position);
		}
	}

	return -1.f;
}

const FCompositeSection& UAnimMontage::GetAnimCompositeSection(int32 SectionIndex) const
{
	check ( CompositeSections.IsValidIndex(SectionIndex) );
	return CompositeSections[SectionIndex];
}

FCompositeSection& UAnimMontage::GetAnimCompositeSection(int32 SectionIndex)
{
	check ( CompositeSections.IsValidIndex(SectionIndex) );
	return CompositeSections[SectionIndex];
}

int32 UAnimMontage::GetSectionIndex(FName InSectionName) const
{
	// I can have operator== to check SectionName, but then I have to construct
	// empty FCompositeSection all the time whenever I search :(
	for (int32 I=0; I<CompositeSections.Num(); ++I)
	{
		if ( CompositeSections[I].SectionName == InSectionName ) 
		{
			return I;
		}
	}

	return INDEX_NONE;
}

FName UAnimMontage::GetSectionName(int32 SectionIndex) const
{
	if ( CompositeSections.IsValidIndex(SectionIndex) )
	{
		return CompositeSections[SectionIndex].SectionName;
	}

	return NAME_None;
}

bool UAnimMontage::IsValidSectionName(FName InSectionName) const
{
	return GetSectionIndex(InSectionName) != INDEX_NONE;
}

bool UAnimMontage::IsValidSectionIndex(int32 SectionIndex) const
{
	return (CompositeSections.IsValidIndex(SectionIndex));
}

void UAnimMontage::GetSectionStartAndEndTime(int32 SectionIndex, float& OutStartTime, float& OutEndTime) const
{
	OutStartTime = 0.f;
	OutEndTime = GetPlayLength();	
	if ( IsValidSectionIndex(SectionIndex) )
	{
		OutStartTime = GetAnimCompositeSection(SectionIndex).GetTime();		
	}

	if ( IsValidSectionIndex(SectionIndex + 1))
	{
		OutEndTime = GetAnimCompositeSection(SectionIndex + 1).GetTime();		
	}
}

float UAnimMontage::GetSectionLength(int32 SectionIndex) const
{
	float StartTime = 0.f;
	float EndTime = GetPlayLength();
	if ( IsValidSectionIndex(SectionIndex) )
	{
		StartTime = GetAnimCompositeSection(SectionIndex).GetTime();		
	}

	if ( IsValidSectionIndex(SectionIndex + 1))
	{
		EndTime = GetAnimCompositeSection(SectionIndex + 1).GetTime();		
	}

	return EndTime - StartTime;
}

#if WITH_EDITOR
int32 UAnimMontage::AddAnimCompositeSection(FName InSectionName, float StartTime)
{
	FCompositeSection NewSection;

	// make sure same name doesn't exists
	if ( InSectionName != NAME_None )
	{
		NewSection.SectionName = InSectionName;
	}
	else
	{
		// just give default name
		NewSection.SectionName = FName(*FString::Printf(TEXT("Section%d"), CompositeSections.Num()+1));
	}

	// we already have that name
	if ( GetSectionIndex(InSectionName)!=INDEX_NONE )
	{
		UE_LOGF(LogAnimMontage, Warning, "AnimCompositeSection : %ls(%ls) already exists. Choose different name.", 
			*NewSection.SectionName.ToString(), *InSectionName.ToString());
		return INDEX_NONE;
	}

	NewSection.Link(this, StartTime);

	// we'd like to sort them in the order of time
	int32 NewSectionIndex = CompositeSections.Add(NewSection);

	// when first added, just make sure to link previous one to add me as next if previous one doesn't have any link
	// it's confusing first time when you add this data
	int32 PrevSectionIndex = NewSectionIndex-1;
	if ( CompositeSections.IsValidIndex(PrevSectionIndex) )
	{
		if (CompositeSections[PrevSectionIndex].NextSectionName == NAME_None)
		{
			CompositeSections[PrevSectionIndex].NextSectionName = InSectionName;
		}
	}

	return NewSectionIndex;
}

bool UAnimMontage::DeleteAnimCompositeSection(int32 SectionIndex)
{
	if ( CompositeSections.IsValidIndex(SectionIndex) )
	{
		CompositeSections.RemoveAt(SectionIndex);
		return true;
	}

	return false;
}
void UAnimMontage::SortAnimCompositeSectionByPos()
{
	// sort them in the order of time
	struct FCompareFCompositeSection
	{
		FORCEINLINE bool operator()( const FCompositeSection &A, const FCompositeSection &B ) const
		{
			return A.GetTime() < B.GetTime();
		}
	};
	CompositeSections.Sort( FCompareFCompositeSection() );
}

void UAnimMontage::RegisterOnMontageChanged(const FOnMontageChanged& Delegate)
{
	OnMontageChanged.Add(Delegate);
}
void UAnimMontage::UnregisterOnMontageChanged(FDelegateUserObject Unregister)
{
	OnMontageChanged.RemoveAll(Unregister);
}
#endif	//WITH_EDITOR

void UAnimMontage::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	BakeTimeStretchCurve();
#endif // WITH_EDITOR
	Super::PreSave(ObjectSaveContext);
}

FFrameRate UAnimMontage::GetSamplingFrameRate() const
{	
	if (CommonTargetFrameRate.IsValid())
	{
		return CommonTargetFrameRate;
	}

	return Super::GetSamplingFrameRate();
}

void UAnimMontage::PostLoad()
{
	// Link notifies before we call the Super::PostLoad (and eventually RefreshCacheData). This is to ensure that branching points get correctly
	// picked up as FAnimNotifyEvent::IsBranchingPoint() relies on the LinkedMontage being valid, and due to an issue where (since deprecated)
	// LinkSequence() was called instead of LinkMontage() there exists content for which LinkedMontage is saved as a null reference.
	for(FAnimNotifyEvent& Notify : Notifies)
	{
#if WITH_EDITORONLY_DATA
		if(Notify.DisplayTime_DEPRECATED != 0.0f)
		{
			Notify.Clear();
			Notify.Link(this, Notify.DisplayTime_DEPRECATED);
		}
		else
#endif
		{
			Notify.Link(this, Notify.GetTime());
		}

		if(Notify.Duration != 0.0f)
		{
			Notify.EndLink.Link(this, Notify.GetTime() + Notify.Duration);
		}
	}
	
	Super::PostLoad();

	// copy deprecated variable to new one, temporary code to keep data copied. Am deleting it right after this
	for ( auto SlotIter = SlotAnimTracks.CreateIterator() ; SlotIter ; ++SlotIter)
	{
		FAnimTrack & Track = (*SlotIter).AnimTrack;
		Track.ValidateSegmentTimes();

		const float CurrentCalculatedLength = CalculateSequenceLength();
		if(!FMath::IsNearlyEqual(CurrentCalculatedLength, GetPlayLength(), UE_KINDA_SMALL_NUMBER))		
		{
			UE_LOGFMT_NSLOC(LogAnimMontage, Display, "AnimMontage", "PostLoad_SequenceLengthMismatch",
				"UAnimMontage::PostLoad: The actual sequence length for {Asset} does not match the length stored in the asset. Please resave the asset.",
				("Asset", UE::FAssetLog(this)));
			SetCompositeLength(CurrentCalculatedLength);
		}

#if WITH_EDITOR
		for (const FAnimSegment& AnimSegment : Track.AnimSegments)
		{
			if(AnimSegment.IsPlayLengthOutOfDate())
			{
				UE_LOGFMT_NSLOC(LogAnimation, Warning, "AnimMontage", "PostLoad_InvalidSectionLength",
					"AnimMontage ({MontageAsset}) contains a Segment for Slot ({SlotName}) for which the playable length {PlayableLength} "
					"is out-of-sync with the represented AnimationSequence its length {AnimSequenceLength} ({AnimSequenceAsset}). Please up-date the segment and resave.",
					("MontageAsset", UE::FAssetLog(this)),
					("SlotName", SlotIter->SlotName),
					("PlayableLength", AnimSegment.AnimEndTime - AnimSegment.AnimStartTime),
					("AnimSequenceLength", AnimSegment.GetAnimReference()->GetPlayLength()),
					("AnimSequenceAsset", UE::FAssetLog(AnimSegment.GetAnimReference())));
			}
		}
#endif
	}

	for(FCompositeSection& Composite : CompositeSections)
	{
#if WITH_EDITORONLY_DATA
		if(Composite.StartTime_DEPRECATED != 0.0f)
		{
			Composite.Clear();
			Composite.Link(this, Composite.StartTime_DEPRECATED);
		}
		else
#endif
		{
			Composite.RefreshSegmentOnLoad();
			Composite.Link(this, Composite.GetTime());
		}
	}

	bool bRootMotionEnabled = bEnableRootMotionTranslation || bEnableRootMotionRotation;

	if (bRootMotionEnabled)
	{
		for (FSlotAnimationTrack& Slot : SlotAnimTracks)
		{
			for (FAnimSegment& Segment : Slot.AnimTrack.AnimSegments)
			{
				if (UAnimSequenceBase* AnimReference = Segment.GetAnimReference())
				{
#if WITH_EDITOR
					if (!AnimReference->GetEnableRootMotionSettingFromMontage())
					{
						UE_LOGF(LogAnimation, Warning, "[Montage %ls] has RootMotionEnabled, but [AnimationSequence %ls] has not been saved after setting the flag. Please open the Montage and the AnimationSequence and save the AnimationSequence as this will generate non determistic cooks.", *GetFullName(), *AnimReference->GetFullName());
					}
#endif // WITH_EDITOR
					AnimReference->EnableRootMotionSettingFromMontage(true, RootMotionRootLock);
				}
			}
		}
	}
	// find preview base pose if it can
#if WITH_EDITORONLY_DATA
	if ( IsValidAdditive() && PreviewBasePose == nullptr )
	{
		for (int32 I=0; I<SlotAnimTracks.Num(); ++I)
		{
			if ( SlotAnimTracks[I].AnimTrack.AnimSegments.Num() > 0 )
			{
				UAnimSequenceBase* SequenceBase = SlotAnimTracks[I].AnimTrack.AnimSegments[0].GetAnimReference();
				UAnimSequence* BaseAdditivePose = (SequenceBase) ? SequenceBase->GetAdditiveBasePose() : nullptr;
				if (BaseAdditivePose)
				{
					PreviewBasePose = BaseAdditivePose;
					MarkPackageDirty();
					break;
				}
			}
		}
	}

	// verify if skeleton is valid, otherwise clear it, this can happen if anim sequence has been modified when this hasn't been loaded. 
	for (int32 I=0; I<SlotAnimTracks.Num(); ++I)
	{
		if ( SlotAnimTracks[I].AnimTrack.AnimSegments.Num() > 0 )
		{
			UAnimSequenceBase* SequenceBase = SlotAnimTracks[I].AnimTrack.AnimSegments[0].GetAnimReference();
			if (SequenceBase && SequenceBase->GetSkeleton() == nullptr)
			{
				SlotAnimTracks[I].AnimTrack.AnimSegments[0].SetAnimReference(nullptr);
				MarkPackageDirty();
				break;
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	// Register Slots w/ Skeleton - to aid deterministic cooking do not do this during cook! 
	if(!GIsCookerLoadingPackage)
	{
		USkeleton* MySkeleton = GetSkeleton();
		if (MySkeleton)
		{
			for (int32 SlotIndex = 0; SlotIndex < SlotAnimTracks.Num(); SlotIndex++)
			{
				FName SlotName = SlotAnimTracks[SlotIndex].SlotName;
				MySkeleton->RegisterSlotNode(SlotName);
			}
		}
	}

	// Convert BranchingPoints to AnimNotifies.
	if (GetLinker() && (GetLinker()->UEVer() < VER_UE4_MONTAGE_BRANCHING_POINT_REMOVAL) )
	{
		ConvertBranchingPointsToAnimNotifies();
	}

#if WITH_EDITORONLY_DATA
	// fix up blending time deprecated variable
	if (BlendInTime_DEPRECATED != -1.f)
	{
		BlendIn.SetBlendTime(BlendInTime_DEPRECATED);
		BlendInTime_DEPRECATED = -1.f;
	}

	if(BlendOutTime_DEPRECATED != -1.f)
	{
		BlendOut.SetBlendTime(BlendOutTime_DEPRECATED);
		BlendOutTime_DEPRECATED = -1.f;
	}
#endif

	// collect markers if it's valid
	CollectMarkers();
}

void UAnimMontage::ConvertBranchingPointsToAnimNotifies()
{
#if WITH_EDITORONLY_DATA
	if (BranchingPoints_DEPRECATED.Num() > 0)
	{
		// Handle deprecated DisplayTime first
		for (auto& BranchingPoint : BranchingPoints_DEPRECATED)
		{
			if (BranchingPoint.DisplayTime_DEPRECATED != 0.0f)
			{
				BranchingPoint.Clear();
				BranchingPoint.Link(this, BranchingPoint.DisplayTime_DEPRECATED);
			}
			else
			{
				BranchingPoint.Link(this, BranchingPoint.GetTime());
			}
		}

		// Then convert to AnimNotifies
		USkeleton * MySkeleton = GetSkeleton();

		// Add a new AnimNotifyTrack, and place all branching points in there.
		int32 TrackIndex = AnimNotifyTracks.Num();

		FAnimNotifyTrack NewItem;
		NewItem.TrackName = *FString::FromInt(TrackIndex + 1);
		NewItem.TrackColor = FLinearColor::White;
		AnimNotifyTracks.Add(NewItem);

		for (auto BranchingPoint : BranchingPoints_DEPRECATED)
		{
			int32 NewNotifyIndex = Notifies.Add(FAnimNotifyEvent());
			FAnimNotifyEvent& NewEvent = Notifies[NewNotifyIndex];
			NewEvent.NotifyName = BranchingPoint.EventName;

			float TriggerTime = BranchingPoint.GetTriggerTime();
			NewEvent.Link(this, TriggerTime);
#if WITH_EDITOR
			NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(CalculateOffsetForNotify(TriggerTime));
#endif
			NewEvent.TrackIndex = TrackIndex;
			NewEvent.Notify = nullptr;
			NewEvent.NotifyStateClass = nullptr;
			NewEvent.bConvertedFromBranchingPoint = true;
			NewEvent.MontageTickType = EMontageNotifyTickType::BranchingPoint;

			// Add as a custom AnimNotify event to Skeleton.
			if (MySkeleton)
			{
				MySkeleton->AnimationNotifies.AddUnique(NewEvent.NotifyName);
			}
		}

		BranchingPoints_DEPRECATED.Empty();
		RefreshBranchingPointMarkers();
	}
#endif
}

void UAnimMontage::RefreshBranchingPointMarkers()
{
	BranchingPointMarkers.Empty();
	BranchingPointStateNotifyIndices.Empty();

	// Verify that we have no overlapping trigger times, this is not supported, and markers would not be triggered then.
	TMap<float, FAnimNotifyEvent*> TriggerTimes;

	int32 NumNotifies = Notifies.Num();
	for (int32 NotifyIndex = 0; NotifyIndex < NumNotifies; NotifyIndex++)
	{
		FAnimNotifyEvent& NotifyEvent = Notifies[NotifyIndex];

		if (NotifyEvent.IsBranchingPoint())
		{
			AddBranchingPointMarker(FBranchingPointMarker(NotifyIndex, NotifyEvent.GetTriggerTime(), EAnimNotifyEventType::Begin), TriggerTimes);

			if (NotifyEvent.NotifyStateClass)
			{
				// Track end point of AnimNotifyStates.
				AddBranchingPointMarker(FBranchingPointMarker(NotifyIndex, NotifyEvent.GetEndTriggerTime(), EAnimNotifyEventType::End), TriggerTimes);

				// Also track AnimNotifyStates separately, so we can tick them between their Begin and End points.
				BranchingPointStateNotifyIndices.Add(NotifyIndex);
			}
		}
	}
	
	if (BranchingPointMarkers.Num() > 0)
	{
		// Sort markers
		struct FCompareNotifyTickMarkersTime
		{
			FORCEINLINE bool operator()(const FBranchingPointMarker &A, const FBranchingPointMarker &B) const
			{
				return A.TriggerTime < B.TriggerTime;
			}
		};

		BranchingPointMarkers.Sort(FCompareNotifyTickMarkersTime());
	}
}

void UAnimMontage::RefreshCacheData()
{
	Super::RefreshCacheData();

	// This gets called whenever notifies are modified in the editor, so refresh our branch list
	RefreshBranchingPointMarkers();
#if WITH_EDITOR
	if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		// This is not needed during post load (as the child montages themselves will handle
		// updating and calling it can cause deterministic cooking issues depending on load order
		PropagateChanges();
	}
#endif // WITH_EDITOR
}

void UAnimMontage::AddBranchingPointMarker(FBranchingPointMarker TickMarker, TMap<float, FAnimNotifyEvent*>& TriggerTimes)
{
	// Add Marker
	BranchingPointMarkers.Add(TickMarker);

	// Check that there is no overlapping marker, as we don't support this.
	// This would mean one of them is not getting triggered!
	FAnimNotifyEvent** FoundNotifyEventPtr = TriggerTimes.Find(TickMarker.TriggerTime);
	if (FoundNotifyEventPtr)
	{
		UE_ASSET_LOG(LogAnimMontage, Warning, this, TEXT("Branching Point '%s' overlaps with '%s' at time: %f. One of them will not get triggered!"),
			*Notifies[TickMarker.NotifyIndex].NotifyName.ToString(), *(*FoundNotifyEventPtr)->NotifyName.ToString(), TickMarker.TriggerTime);
	}
	else
	{
		TriggerTimes.Add(TickMarker.TriggerTime, &Notifies[TickMarker.NotifyIndex]);
	}
}

const FBranchingPointMarker* UAnimMontage::FindFirstBranchingPointMarker(float StartTrackPos, float EndTrackPos) const
{
	if (BranchingPointMarkers.Num() > 0)
	{
		const bool bSearchBackwards = (EndTrackPos < StartTrackPos);
		if (!bSearchBackwards)
		{
			for (int32 Index = 0; Index < BranchingPointMarkers.Num(); Index++)
			{
				const FBranchingPointMarker& Marker = BranchingPointMarkers[Index];
				if (Marker.TriggerTime <= StartTrackPos)
				{
					continue;
				}
				if (Marker.TriggerTime > EndTrackPos)
				{
					break;
				}
				return &Marker;
			}
		}
		else
		{
			for (int32 Index = BranchingPointMarkers.Num() - 1; Index >= 0; Index--)
			{
				const FBranchingPointMarker& Marker = BranchingPointMarkers[Index];
				if (Marker.TriggerTime >= StartTrackPos)
				{
					continue;
				}
				if (Marker.TriggerTime < EndTrackPos)
				{
					break;
				}
				return &Marker;
			}
		}
	}
	return nullptr;
}

void UAnimMontage::FilterOutNotifyBranchingPoints(TArray<const FAnimNotifyEvent*>& InAnimNotifies)
{
	for (int32 Index = InAnimNotifies.Num()-1; Index >= 0; Index--)
	{
		if (InAnimNotifies[Index]->IsBranchingPoint())
		{
			InAnimNotifies.RemoveAt(Index, 1);
		}
	}
}

void UAnimMontage::FilterOutNotifyBranchingPoints(TArray<FAnimNotifyEventReference>& InAnimNotifies)
{
	for (int32 Index = InAnimNotifies.Num() - 1; Index >= 0; Index--)
	{
		if(const FAnimNotifyEvent* Notify = InAnimNotifies[Index].GetNotify())
		if (!Notify || Notify->IsBranchingPoint())
		{
			InAnimNotifies.RemoveAt(Index, 1);
		}
	}
}

#if WITH_EDITOR
void UAnimMontage::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	// It is unclear if CollectMarkers should be here or in RefreshCacheData
	if (SyncGroup != NAME_None)
	{
		CollectMarkers();
	}

	UpdateCommonTargetFrameRate();

	PropagateChanges();
}

void UAnimMontage::PropagateChanges()
{
	// @note propagate to children
	// this isn't that slow yet, but if this gets slow, we'll have to do guid method
	if (ChildrenAssets.Num() > 0)
	{
		for (UAnimationAsset* Child : ChildrenAssets)
		{
			if (Child)
			{
				Child->UpdateParentAsset();
			}
		}
	}
}
#endif // WITH_EDITOR

void UAnimMontage::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CompositeSections.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SlotAnimTracks.GetAllocatedSize());
	for (FSlotAnimationTrack& Slot : SlotAnimTracks)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Slot.AnimTrack.GetTotalBytesUsed());
	}
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(BranchingPointMarkers.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(BranchingPointStateNotifyIndices.GetAllocatedSize());
}

bool UAnimMontage::IsValidAdditive() const
{
	// if first one is additive, this is additive
	if ( SlotAnimTracks.Num() > 0 )
	{
		for (int32 I=0; I<SlotAnimTracks.Num(); ++I)
		{
			if (!SlotAnimTracks[I].AnimTrack.IsAdditive())
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

bool UAnimMontage::IsValidAdditiveSlot(const FName& SlotNodeName) const
{
	// if first one is additive, this is additive
	if ( SlotAnimTracks.Num() > 0 )
	{
		for (int32 I=0; I<SlotAnimTracks.Num(); ++I)
		{
			if (SlotAnimTracks[I].SlotName == SlotNodeName)
			{
				return SlotAnimTracks[I].AnimTrack.IsAdditive();
			}
		}
	}

	return false;
}

EAnimEventTriggerOffsets::Type UAnimMontage::CalculateOffsetFromSections(float Time) const
{
	for(auto Iter = CompositeSections.CreateConstIterator(); Iter; ++Iter)
	{
		float SectionTime = Iter->GetTime();
		if(FMath::IsNearlyEqual(SectionTime,Time))
		{
			return EAnimEventTriggerOffsets::OffsetBefore;
		}
	}
	return EAnimEventTriggerOffsets::NoOffset;
}

bool FAnimMontageInstance::ValidateInstanceAfterNotifyState(const TWeakObjectPtr<UAnimInstance>& InAnimInstance, const UAnimNotifyState* InNotifyStateClass)
{
	// An owning instance should never be invalid after a notify call, since it's where the montage instance lives
	if (!InAnimInstance.IsValid())
	{
		ensureMsgf(false, TEXT("Invalid anim instance after triggering notify: %s"), *GetNameSafe(InNotifyStateClass));
		return false;
	}

	// Montage instances array should never be empty after a notify state
	if (InAnimInstance->MontageInstances.Num() == 0)
	{
		ensureMsgf(false, TEXT("Montage instances empty on AnimInstance(%s) after calling notify:  %s"), *GetNameSafe(InAnimInstance.Get()), *GetNameSafe(InNotifyStateClass));
		return false;
	}

	return true;
}

#if WITH_EDITOR
EAnimEventTriggerOffsets::Type UAnimMontage::CalculateOffsetForNotify(float NotifyDisplayTime) const
{
	EAnimEventTriggerOffsets::Type Offset = Super::CalculateOffsetForNotify(NotifyDisplayTime);
	if(Offset == EAnimEventTriggerOffsets::NoOffset)
	{
		Offset = CalculateOffsetFromSections(NotifyDisplayTime);
	}
	return Offset;
}
#endif

bool UAnimMontage::HasRootMotion() const
{
	for (const FSlotAnimationTrack& Track : SlotAnimTracks)
	{
		if (Track.AnimTrack.HasRootMotion())
		{
			return true;
		}
	}
	return false;
}

/** Extract RootMotion Transform from a contiguous Track position range.
 * *CONTIGUOUS* means that if playing forward StartTractPosition < EndTrackPosition.
 * No wrapping over if looping. No jumping across different sections.
 * So the AnimMontage has to break the update into contiguous pieces to handle those cases.
 *
 * This does handle Montage playing backwards (StartTrackPosition > EndTrackPosition).
 *
 * It will break down the range into steps if needed to handle looping animations, or different animations.
 * These steps will be processed sequentially, and output the RootMotion transform in component space.
 */
FTransform UAnimMontage::ExtractRootMotionFromTrackRange(float StartTrackPosition, float EndTrackPosition, const FAnimExtractContext& Context) const
{
	FRootMotionMovementParams RootMotion;

	// For now assume Root Motion only comes from first track.
	if( SlotAnimTracks.Num() > 0 )
	{
		const FAnimTrack& SlotAnimTrack = SlotAnimTracks[0].AnimTrack;

		// Get RootMotion pieces from this track.
		// We can deal with looping animations, or multiple animations. So we break those up into sequential operations.
		// (Animation, StartFrame, EndFrame) so we can then extract root motion sequentially.
		ExtractRootMotionFromTrack(SlotAnimTrack, StartTrackPosition, EndTrackPosition, Context, RootMotion);

	}

	UE_LOGF(LogRootMotion, Log,  "\tUAnimMontage::ExtractRootMotionForTrackRange RootMotionTransform: Translation: %ls, Rotation: %ls"
		, *RootMotion.GetRootMotionTransform().GetTranslation().ToCompactString()
		, *RootMotion.GetRootMotionTransform().GetRotation().Rotator().ToCompactString()
		);

	return RootMotion.GetRootMotionTransform();
}

/** Get Montage's Group Name */
FName UAnimMontage::GetGroupName() const
{
	USkeleton* MySkeleton = GetSkeleton();
	if (MySkeleton && (SlotAnimTracks.Num() > 0))
	{
		return MySkeleton->GetSlotGroupName(SlotAnimTracks[0].SlotName);
	}

	return FAnimSlotGroup::DefaultGroupName;
}

bool UAnimMontage::HasValidSlotSetup() const
{
	// We only need to worry about this if we have multiple tracks.
	// Montages with a single track will always have a valid slot setup.
	int32 NumAnimTracks = SlotAnimTracks.Num();
	if (NumAnimTracks > 1)
	{
		USkeleton* MySkeleton = GetSkeleton();
		if (MySkeleton)
		{
			FName MontageGroupName = GetGroupName();
			TArray<FName> UniqueSlotNameList;
			UniqueSlotNameList.Add(SlotAnimTracks[0].SlotName);

			for (int32 TrackIndex = 1; TrackIndex < NumAnimTracks; TrackIndex++)
			{
				// Verify that slot names are unique.
				FName CurrentSlotName = SlotAnimTracks[TrackIndex].SlotName;
				bool bSlotNameAlreadyInUse = UniqueSlotNameList.Contains(CurrentSlotName);
				if (!bSlotNameAlreadyInUse)
				{
					UniqueSlotNameList.Add(CurrentSlotName);
				}
				else
				{
					UE_LOGF(LogAnimMontage, Warning, "Montage '%ls' not properly setup. Slot named '%ls' is already used in this Montage. All slots must be unique",
						*GetFullName(), *CurrentSlotName.ToString());
					return false;
				}

				// Verify that all slots belong to the same group.
				FName CurrentSlotGroupName = MySkeleton->GetSlotGroupName(CurrentSlotName);
				bool bDifferentGroupName = (CurrentSlotGroupName != MontageGroupName);
				if (bDifferentGroupName)
				{
					UE_LOGF(LogAnimMontage, Warning, "Montage '%ls' not properly setup. Slot's group '%ls' is different than the Montage's group '%ls'. All slots must belong to the same group.",
						*GetFullName(), *CurrentSlotGroupName.ToString(), *MontageGroupName.ToString());
					return false;
				}
			}
		}
	}

	return true;
}

float UAnimMontage::CalculateSequenceLength()
{
	float CalculatedSequenceLength = 0.f;
	for (auto Iter = SlotAnimTracks.CreateIterator(); Iter; ++Iter)
	{
		FSlotAnimationTrack& SlotAnimTrack = (*Iter);
		if (SlotAnimTrack.AnimTrack.AnimSegments.Num() > 0)
		{
			CalculatedSequenceLength = FMath::Max(CalculatedSequenceLength, SlotAnimTrack.AnimTrack.GetLength());
		}
	}
	return CalculatedSequenceLength;
}

const TArray<class UAnimMetaData*> UAnimMontage::GetSectionMetaData(FName SectionName, bool bIncludeSequence/*=true*/, FName SlotName /*= NAME_None*/)
{
	TArray<class UAnimMetaData*> MetadataList;
	bool bShouldIIncludeSequence = bIncludeSequence;

	for (int32 SectionIndex = 0; SectionIndex < CompositeSections.Num(); ++SectionIndex)
	{
		const auto& CurSection = CompositeSections[SectionIndex];
		if (SectionName == NAME_None || CurSection.SectionName == SectionName)
		{
			// add to the list
			MetadataList.Append(CurSection.GetMetaData());

			if (bShouldIIncludeSequence)
			{
				if (SectionName == NAME_None)
				{
					for (auto& SlotIter : SlotAnimTracks)
					{
						if (SlotName == NAME_None || SlotIter.SlotName == SlotName)
						{
							// now add the animations within this section
							for (auto& SegmentIter : SlotIter.AnimTrack.AnimSegments)
							{
								if (UAnimSequenceBase* AnimReference = SegmentIter.GetAnimReference())
								{
									// only add unique here
									TArray<UAnimMetaData*> RefMetadata = AnimReference->GetMetaData();

									for (auto& RefData : RefMetadata)
									{
										MetadataList.AddUnique(RefData);
									}
								}
							}
						}
					}

					// if section name == None, we only grab slots once
					// otherwise, it will grab multiple times
					bShouldIIncludeSequence = false;
				}
				else
				{
					float SectionStartTime = 0.f, SectionEndTime = 0.f;
					GetSectionStartAndEndTime(SectionIndex, SectionStartTime, SectionEndTime);
					for (auto& SlotIter : SlotAnimTracks)
					{
						if (SlotName == NAME_None || SlotIter.SlotName == SlotName)
						{
							// now add the animations within this section
							for (auto& SegmentIter : SlotIter.AnimTrack.AnimSegments)
							{
								if (SegmentIter.IsIncluded(SectionStartTime, SectionEndTime))
								{
									if (UAnimSequenceBase* AnimReference = SegmentIter.GetAnimReference())
									{
										// only add unique here
										TArray<UAnimMetaData*> RefMetadata = AnimReference->GetMetaData();

										for (auto& RefData : RefMetadata)
										{
											MetadataList.AddUnique(RefData);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	return MetadataList;
}

#if WITH_EDITOR
bool UAnimMontage::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive /*= true*/)
{
	Super::GetAllAnimationSequencesReferred(AnimationAssets, bRecursive);

	for (auto Iter = SlotAnimTracks.CreateConstIterator(); Iter; ++Iter)
	{
		const FSlotAnimationTrack& Track = (*Iter);
		Track.AnimTrack.GetAllAnimationSequencesReferred(AnimationAssets, bRecursive);
	}

	if (PreviewBasePose)
	{
		PreviewBasePose->HandleAnimReferenceCollection(AnimationAssets, bRecursive);
	}

	return (AnimationAssets.Num() > 0);
}

void UAnimMontage::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	Super::ReplaceReferredAnimations(ReplacementMap);

	for (auto Iter = SlotAnimTracks.CreateIterator(); Iter; ++Iter)
	{
		FSlotAnimationTrack& Track = (*Iter);
		Track.AnimTrack.ReplaceReferredAnimations(ReplacementMap);
	}

	if (PreviewBasePose)
	{
		UAnimSequence* const* ReplacementAsset = (UAnimSequence*const*)ReplacementMap.Find(PreviewBasePose);
		if (ReplacementAsset)
		{
			PreviewBasePose = *ReplacementAsset;
			PreviewBasePose->ReplaceReferredAnimations(ReplacementMap);
		}
	}
}

void UAnimMontage::UpdateLinkableElements()
{
	// Update all linkable elements
	for (FCompositeSection& Section : CompositeSections)
	{
		Section.Update();
	}

	for (FAnimNotifyEvent& Notify : Notifies)
	{
		Notify.Update();
		Notify.RefreshTriggerOffset(CalculateOffsetForNotify(Notify.GetTime()));

		Notify.EndLink.Update();
		Notify.RefreshEndTriggerOffset(CalculateOffsetForNotify(Notify.EndLink.GetTime()));
	}
}

void UAnimMontage::UpdateLinkableElements(int32 SlotIdx, int32 SegmentIdx)
{
	for (FCompositeSection& Section : CompositeSections)
	{
		if (Section.GetSlotIndex() == SlotIdx && Section.GetSegmentIndex() == SegmentIdx)
		{
			// Update the link
			Section.Update();
		}
	}

	for (FAnimNotifyEvent& Notify : Notifies)
	{
		if (Notify.GetSlotIndex() == SlotIdx && Notify.GetSegmentIndex() == SegmentIdx)
		{
			Notify.Update();
			Notify.RefreshTriggerOffset(CalculateOffsetForNotify(Notify.GetTime()));
		}

		if (Notify.EndLink.GetSlotIndex() == SlotIdx && Notify.EndLink.GetSegmentIndex() == SegmentIdx)
		{
			Notify.EndLink.Update();
			Notify.RefreshEndTriggerOffset(CalculateOffsetForNotify(Notify.EndLink.GetTime()));
		}
	}
}

void UAnimMontage::RefreshParentAssetData()
{
	Super::RefreshParentAssetData();

	UAnimMontage* ParentMontage = CastChecked<UAnimMontage>(ParentAsset);

	BlendIn = ParentMontage->BlendIn;
	BlendOut = ParentMontage->BlendOut;
	BlendOutTriggerTime = ParentMontage->BlendOutTriggerTime;
	SyncGroup = ParentMontage->SyncGroup;
	SyncSlotIndex = ParentMontage->SyncSlotIndex;

	MarkerData = ParentMontage->MarkerData;
	CompositeSections = ParentMontage->CompositeSections;
	SlotAnimTracks = ParentMontage->SlotAnimTracks;

	PreviewBasePose = ParentMontage->PreviewBasePose;
	BranchingPointMarkers = ParentMontage->BranchingPointMarkers;
	BranchingPointStateNotifyIndices = ParentMontage->BranchingPointStateNotifyIndices;

	for (int32 SlotIdx = 0; SlotIdx < SlotAnimTracks.Num(); ++SlotIdx)
	{
		FSlotAnimationTrack& SlotTrack = SlotAnimTracks[SlotIdx];
		
		for (int32 SegmentIdx = 0; SegmentIdx < SlotTrack.AnimTrack.AnimSegments.Num(); ++SegmentIdx)
		{
			FAnimSegment& Segment = SlotTrack.AnimTrack.AnimSegments[SegmentIdx];
			FAnimSegment& ParentSegment = ParentMontage->SlotAnimTracks[SlotIdx].AnimTrack.AnimSegments[SegmentIdx];
			UAnimSequenceBase* SourceReference = Segment.GetAnimReference();
			UAnimSequenceBase* TargetReference = Cast<UAnimSequenceBase>(AssetMappingTable->GetMappedAsset(SourceReference));
			Segment.SetAnimReference(TargetReference);

			float LengthChange = FMath::IsNearlyZero(SourceReference->GetPlayLength()) ? 0.f : TargetReference->GetPlayLength() / SourceReference->GetPlayLength();
			float RateChange = FMath::IsNearlyZero(SourceReference->RateScale) ? 0.f : FMath::Abs(TargetReference->RateScale / SourceReference->RateScale);
			float TotalRateChange = FMath::IsNearlyZero(RateChange)? 0.f : (LengthChange / RateChange);
			Segment.AnimPlayRate *= TotalRateChange;
			Segment.AnimStartTime *= LengthChange;
			Segment.AnimEndTime *= LengthChange;
		}
	}

	UpdateCommonTargetFrameRate();

	OnMontageChanged.Broadcast();
}

#endif

FString MakePositionMessage(const FMarkerSyncAnimPosition& Position)
{
	return FString::Printf(TEXT("Names(PrevName: %s | NextName: %s) PosBetweenMarkers: %.2f"), *Position.PreviousMarkerName.ToString(), *Position.NextMarkerName.ToString(), Position.PositionBetweenMarkers);
}

void UAnimMontage::TickAssetPlayer(FAnimTickRecord& Instance, struct FAnimNotifyQueue& NotifyQueue, FAnimAssetTickContext& Context) const
{
	bool bRecordNeedsResetting = true;

	// nothing has to happen here
	// we just have to make sure we set Context data correct
	//if (ensure (Context.IsLeader()))
	if (Context.IsLeader())
	{
		check(Instance.DeltaTimeRecord);
		const float CurrentTime = Instance.Montage.CurrentPosition;
		const float PreviousTime = Instance.DeltaTimeRecord->GetPrevious();
		const float MoveDelta = Instance.DeltaTimeRecord->Delta;

		// Update context's data for followers to use.
		Context.SetLeaderDelta(MoveDelta);
		Context.SetPreviousAnimationPositionRatio(PreviousTime / GetPlayLength());

		if (MoveDelta != 0.f)
		{
			if (Instance.bCanUseMarkerSync && Instance.MarkerTickRecord && Context.CanUseMarkerPosition())
			{
				FMarkerTickRecord* MarkerTickRecord = Instance.MarkerTickRecord;
				FMarkerTickContext& MarkerTickContext = Context.MarkerTickContext;
				const bool bIsMarkerTickRecordValid = MarkerTickRecord->IsValid(Instance.bLooping);
				
				// Store the sync anim position BEFORE the asset has being ticked.
				if (bIsMarkerTickRecordValid)
				{
					MarkerTickContext.SetMarkerSyncStartPosition(GetMarkerSyncPositionFromMarkerIndicies(MarkerTickRecord->PreviousMarker.MarkerIndex, MarkerTickRecord->NextMarker.MarkerIndex, PreviousTime, nullptr));
				}
				else
				{
					// only thing is that passed markers won't work in this frame. To do that, I have to figure out how it jumped from where to where, 
					FMarkerPair PreviousMarker;
					FMarkerPair NextMarker;
					GetMarkerIndicesForTime(PreviousTime, false, MarkerTickContext.GetValidMarkerNames(), PreviousMarker, NextMarker);
					MarkerTickContext.SetMarkerSyncStartPosition(GetMarkerSyncPositionFromMarkerIndicies(PreviousMarker.MarkerIndex, NextMarker.MarkerIndex, PreviousTime, nullptr));
				}

				// Advance as leader.
				// @todo this won't work well once we start jumping
				// only thing is that passed markers won't work in this frame. To do that, I have to figure out how it jumped from where to where, 
				GetMarkerIndicesForTime(CurrentTime, false, MarkerTickContext.GetValidMarkerNames(), MarkerTickRecord->PreviousMarker, MarkerTickRecord->NextMarker);
				bRecordNeedsResetting = false; // we have updated it now, no need to reset.

				// Store the sync anim position AFTER the asset has being ticked.
				MarkerTickContext.SetMarkerSyncEndPosition(GetMarkerSyncPositionFromMarkerIndicies(MarkerTickRecord->PreviousMarker.MarkerIndex, MarkerTickRecord->NextMarker.MarkerIndex, CurrentTime, nullptr));

				MarkerTickContext.MarkersPassedThisTick = *Instance.Montage.MarkersPassedThisTick;

#if DO_CHECK
				// The marker tick record gets invalidated when the montage position is set externally and due to this change we cannot assume
				// its sync positions will be the same as the previous tick. 
				if (MarkerTickContext.MarkersPassedThisTick.Num() == 0 && bIsMarkerTickRecordValid)
				{
					const FMarkerSyncAnimPosition& StartPosition = MarkerTickContext.GetMarkerSyncStartPosition();
					const FMarkerSyncAnimPosition& EndPosition = MarkerTickContext.GetMarkerSyncEndPosition();
					checkf(StartPosition.NextMarkerName == EndPosition.NextMarkerName, TEXT("StartPosition %s\nEndPosition %s\nPrevTime to CurrentTimeAsset: %.3f - %.3f Delta: %.3f\nAsset = %s"), *MakePositionMessage(StartPosition), *MakePositionMessage(EndPosition), PreviousTime, CurrentTime, MoveDelta, *Instance.SourceAsset->GetFullName());
					checkf(StartPosition.PreviousMarkerName == EndPosition.PreviousMarkerName, TEXT("StartPosition %s\nEndPosition %s\nPrevTime - CurrentTimeAsset: %.3f - %.3f Delta: %.3f\nAsset = %s"), *MakePositionMessage(StartPosition), *MakePositionMessage(EndPosition), PreviousTime, CurrentTime, MoveDelta, *Instance.SourceAsset->GetFullName());
				}
#endif

				UE_LOGF(LogAnimMarkerSync, Log, "Montage Leading SyncGroup: %ls(%ls) Start [%ls], End [%ls]", *GetNameSafe(this), *SyncGroup.ToString(), *MarkerTickContext.GetMarkerSyncStartPosition().ToString(), *MarkerTickContext.GetMarkerSyncEndPosition().ToString());
			}
		}
		
		// Update context's position for followers to use.
		Context.SetAnimationPositionRatio(CurrentTime / GetPlayLength());
	}

	// Reset record if needed.
	if (bRecordNeedsResetting && Instance.MarkerTickRecord)
	{
		Instance.MarkerTickRecord->Reset();
	}
}

void UAnimMontage::CollectMarkers()
{
	MarkerData.AuthoredSyncMarkers.Reset();

	// We want to make sure anim reference actually contains markers
	if (SyncGroup != NAME_None)
	{
		if (SlotAnimTracks.IsValidIndex(SyncSlotIndex))
		{
			const FAnimTrack& AnimTrack = SlotAnimTracks[SyncSlotIndex].AnimTrack;
			for (const auto& Seg : AnimTrack.AnimSegments)
			{
				const UAnimSequence* Sequence = Cast<UAnimSequence>(Seg.GetAnimReference());
				if (Sequence && Sequence->AuthoredSyncMarkers.Num() > 0)
				{
					// @todo this won't work well if you have starttime < end time and it does have negative playrate
					for (const auto& Marker : Sequence->AuthoredSyncMarkers)
					{
						if (Marker.Time >= Seg.AnimStartTime && Marker.Time <= Seg.AnimEndTime)
						{
							const float TotalSegmentLength = (Seg.AnimEndTime - Seg.AnimStartTime)*Seg.AnimPlayRate;
							// i don't think we can do negative in this case
							ensure(TotalSegmentLength >= 0.f);

							// now add to the list
							for (int32 LoopCount = 0; LoopCount < Seg.LoopingCount; ++LoopCount)
							{
								FAnimSyncMarker NewMarker;

								NewMarker.Time = Seg.StartPos + (Marker.Time - Seg.AnimStartTime)*Seg.AnimPlayRate + TotalSegmentLength*LoopCount;
								NewMarker.MarkerName = Marker.MarkerName;
								MarkerData.AuthoredSyncMarkers.Add(NewMarker);
							}
						}
					}
				}
			}

			MarkerData.CollectUniqueNames();
		}
		else
		{
			UE_LOGF(LogAnimMontage, Warning, "Montage's sync slot track index is invalid. Make sure to use a valid slot track index, otherwise this Montage will use old sync markers or not use marker-based syncing at all.")
		}
	}
}

void UAnimMontage::GetMarkerIndicesForTime(float CurrentTime, bool bLooping, const TArray<FName>& ValidMarkerNames, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker) const
{
	MarkerData.GetMarkerIndicesForTime(CurrentTime, bLooping, ValidMarkerNames, OutPrevMarker, OutNextMarker, GetPlayLength());
}

FMarkerSyncAnimPosition UAnimMontage::GetMarkerSyncPositionFromMarkerIndicies(int32 PrevMarker, int32 NextMarker, float CurrentTime, const UMirrorDataTable* MirrorTable) const
{
	return MarkerData.GetMarkerSyncPositionFromMarkerIndicies(PrevMarker, NextMarker, CurrentTime, GetPlayLength(), MirrorTable);
}

void UAnimMontage::InvalidateRecursiveAsset()
{
	for (FSlotAnimationTrack& SlotTrack : SlotAnimTracks)
	{
		SlotTrack.AnimTrack.InvalidateRecursiveAsset(this);
	}
}

bool UAnimMontage::ContainRecursive(TArray<UAnimCompositeBase*>& CurrentAccumulatedList) 
{
	// am I included already?
	if (CurrentAccumulatedList.Contains(this))
	{
		return true;
	}

	// otherwise, add myself to it
	CurrentAccumulatedList.Add(this);

	for (FSlotAnimationTrack& SlotTrack : SlotAnimTracks)
	{
		// otherwise send to animation track
		if (SlotTrack.AnimTrack.ContainRecursive(CurrentAccumulatedList))
		{
			return true;
		}
	}

	return false;
}

void UAnimMontage::SetCompositeLength(float InLength)
{
#if WITH_EDITOR
	if (!GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		const FFrameTime LengthInFrameTime = DataModelInterface->GetFrameRate().AsFrameTime(InLength);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Controller->SetNumberOfFrames(LengthInFrameTime.RoundToFrame());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
#endif
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SequenceLength = InLength;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// MontageInstance
/////////////////////////////////////////////////////////////////////////////////////////////

FAnimMontageInstance::FAnimMontageInstance()
	: Montage(nullptr)
	, bPlaying(false)
	, DefaultBlendTimeMultiplier(1.0f)
	, bDidUseMarkerSyncThisTick(false)
	, AnimInstance(nullptr)
	, InstanceID(INDEX_NONE)
	, Position(0.f)
	, PlayRate(1.f)
	, bInterrupted(false)
	, PreviousWeight(0.f)
	, NotifyWeight(0.f)
	, BlendStartAlpha(0.0f)
	, SyncGroupName(NAME_None)
	, ActiveBlendProfile(nullptr)
	, ActiveBlendProfileMode(EBlendProfileMode::TimeFactor)
	, DisableRootMotionCount(0)
	, MontageSyncLeader(nullptr)
	, MontageSyncUpdateFrameCounter(INDEX_NONE)
{
}

FAnimMontageInstance::FAnimMontageInstance(UAnimInstance * InAnimInstance)
	: Montage(nullptr)
	, bPlaying(false)
	, DefaultBlendTimeMultiplier(1.0f)
	, bDidUseMarkerSyncThisTick(false)
	, bEnableAutoBlendOut(true)
	, AnimInstance(InAnimInstance)
	, InstanceID(INDEX_NONE)
	, Position(0.f)
	, PlayRate(1.f)
	, bInterrupted(false)
	, PreviousWeight(0.f)
	, NotifyWeight(0.f)
	, BlendStartAlpha(0.0f)
	, SyncGroupName(NAME_None)
	, ActiveBlendProfile(nullptr)
	, ActiveBlendProfileMode(EBlendProfileMode::TimeFactor)
	, DisableRootMotionCount(0)
	, MontageSyncLeader(nullptr)
	, MontageSyncUpdateFrameCounter(INDEX_NONE)
{
}

void FAnimMontageInstance::Play(float InPlayRate)
{
	FMontageBlendSettings BlendInSettings;

	// Fill settings from our Montage asset
	if (Montage)
	{
		BlendInSettings.Blend = Montage->BlendIn;
		BlendInSettings.BlendMode = Montage->BlendModeIn;
		BlendInSettings.BlendProfile = Montage->BlendProfileIn;
	}

	Play(InPlayRate, BlendInSettings);
}

void FAnimMontageInstance::Play(float InPlayRate, const FMontageBlendSettings& BlendInSettings)
{
	bPlaying = true;
	PlayRate = InPlayRate;

	// if this doesn't exist, nothing works
	check(Montage);
	
	// Inertialization
	FAlphaBlendArgs BlendInArgs = BlendInSettings.Blend;
	if (AnimInstance.IsValid() && BlendInSettings.BlendMode == EMontageBlendMode::Inertialization)
	{
		FInertializationRequest Request;
		Request.Duration = BlendInArgs.BlendTime;
		Request.BlendMode = BlendInArgs.BlendOption;
		Request.CustomBlendCurve = BlendInArgs.CustomCurve;
		Request.BlendProfile = BlendInSettings.BlendProfile;
		Request.bUseBlendMode = (BlendInArgs.BlendOption != EAlphaBlendOption::Linear) || (BlendInArgs.CustomCurve != nullptr);

		// Request new inertialization for new montage's group name
		// If there is an existing inertialization request, we overwrite that here.
		AnimInstance->RequestMontageInertialization(Montage, Request);

		// When using inertialization, we need to instantly blend in.
		BlendInArgs.BlendTime = 0.0f;
	}

	// set blend option
	float CurrentWeight = Blend.GetBlendedValue();
	InitializeBlend(FAlphaBlend(BlendInArgs));
	BlendStartAlpha = Blend.GetAlpha();
	Blend.SetBlendTime(BlendInArgs.BlendTime * DefaultBlendTimeMultiplier);
	Blend.SetValueRange(CurrentWeight, 1.f);
	bEnableAutoBlendOut = Montage->bEnableAutoBlendOut;

	ActiveBlendProfile = BlendInSettings.BlendProfile;
}

void FAnimMontageInstance::InitializeBlend(const FAlphaBlend& InAlphaBlend)
{
	Blend.SetBlendOption(InAlphaBlend.GetBlendOption());
	Blend.SetCustomCurve(InAlphaBlend.GetCustomCurve());
	Blend.SetBlendTime(InAlphaBlend.GetBlendTime());
}

void FAnimMontageInstance::Stop(const FMontageBlendSettings& InBlendOutSettings, bool bInterrupt)
{
	if (Montage)
	{
		UE_LOGF(LogAnimMontage, Verbose, "Montage.Stop Before: AnimMontage: %ls,  (DesiredWeight:%0.2f, Weight:%0.2f)",
			*Montage->GetName(), GetDesiredWeight(), GetWeight());
	}

	// overwrite bInterrupted if it hasn't already interrupted
	// once interrupted, you don't go back to non-interrupted
	if (!bInterrupted && bInterrupt)
	{
		bInterrupted = bInterrupt;
	}

	// if it hasn't stopped, stop now
	if (IsStopped() == false)
	{
		// If we are using Inertial Blend, blend time should be 0 to instantly stop the montage.
		FAlphaBlendArgs BlendOutArgs = InBlendOutSettings.Blend;
		const bool bShouldInertialize = InBlendOutSettings.BlendMode == EMontageBlendMode::Inertialization;
		BlendOutArgs.BlendTime = bShouldInertialize ? 0.0f : BlendOutArgs.BlendTime;

		// do not use default Montage->BlendOut 
		// depending on situation, the BlendOut time can change 
		InitializeBlend(FAlphaBlend(BlendOutArgs));
		BlendStartAlpha = Blend.GetAlpha();
		Blend.SetDesiredValue(0.f);
		Blend.Update(0.0f);

		// Only change the active blend profile if the montage isn't stopped. This is to prevent pops on a sudden blend profile switch
		ActiveBlendProfile = InBlendOutSettings.BlendProfile;

		if(Montage)
		{
			if (UAnimInstance* Inst = AnimInstance.Get())
			{
				FAnimMontageInstance* SelfPtr = this;
				uint32 LastMontageFlushFrame = Inst->GetLastMontageFlushFrame();
				
				// Let AnimInstance know we are being stopped.
				Inst->OnMontageInstanceStopped(*this);
				Inst->QueueMontageBlendingOutEvent(FQueuedMontageBlendingOutEvent(Montage, bInterrupted, OnMontageBlendingOutStarted));
				
				// UninitializeAnimation() was called for the our animation instance when executing the blending out event(s).
				if (!Inst->IsInitialized() && MontageCVars::bEarlyOutMontageWhenUninitialized)
				{
					return;
				}

				// Prone to have being free self from trigger montage events.
				if (LastMontageFlushFrame != Inst->GetLastMontageFlushFrame() && Inst->MontageInstances.Find(SelfPtr) == INDEX_NONE && MontageCVars::bEarlyOutMontageWhenUninitialized)
				{
					return;
				}
				
				if (bShouldInertialize)
				{
					FInertializationRequest Request;
					Request.Duration = InBlendOutSettings.Blend.BlendTime;
					Request.BlendMode = InBlendOutSettings.Blend.BlendOption;
					Request.CustomBlendCurve = InBlendOutSettings.Blend.CustomCurve;
					Request.BlendProfile = InBlendOutSettings.BlendProfile;
					Request.bUseBlendMode = (InBlendOutSettings.Blend.BlendOption != EAlphaBlendOption::Linear) || (InBlendOutSettings.Blend.CustomCurve != nullptr);

					// Send the inertial blend request to the anim instance
					Inst->RequestMontageInertialization(Montage, Request);
				}
			}
		}
	}
	else
	{
		// it is already stopped, but new montage blendtime is shorter than what 
		// I'm blending out, that means this needs to readjust blendtime
		// that way we don't accumulate old longer blendtime for newer montage to play
		if (InBlendOutSettings.Blend.BlendTime < Blend.GetBlendTime())
		{
			// I don't know if also using inBlendOut is better than
			// currently set up blend option, but it might be worse to switch between 
			// blending out, but it is possible options in the future
			Blend.SetBlendTime(InBlendOutSettings.Blend.BlendTime);
			BlendStartAlpha = Blend.GetAlpha();
			// have to call this again to restart blending with new blend time
			// we don't change blend options
			Blend.SetDesiredValue(0.f);
		}
	}

	// if blending time < 0.f
	// set the playing to be false
	// @todo is this better to be IsComplete? 
	// or maybe we need this for if somebody sets blend time to be 0.f
	if (Blend.GetBlendTime() <= 0.0f)
	{
		bPlaying = false;
	}

	if (Montage != nullptr)
	{
		UE_LOGF(LogAnimMontage, Verbose, "Montage.Stop After: AnimMontage: %ls,  (DesiredWeight:%0.2f, Weight:%0.2f)",
			*Montage->GetName(), GetDesiredWeight(), GetWeight());
	}
}

void FAnimMontageInstance::Stop(const FAlphaBlend& InBlendOut, bool bInterrupt/*=true*/)
{
	FMontageBlendSettings BlendOutSettings;
	BlendOutSettings.Blend = InBlendOut;

	// Fill our other settings from the montage asset
	if (Montage)
	{
		BlendOutSettings.BlendMode = Montage->BlendModeOut;
		BlendOutSettings.BlendProfile = Montage->BlendProfileOut;
	}

	Stop(BlendOutSettings, bInterrupt);
}

void FAnimMontageInstance::Pause()
{
	bPlaying = false;
}

void FAnimMontageInstance::Initialize(class UAnimMontage * InMontage)
{
	// Generate unique ID for this instance
	static int32 IncrementInstanceID = 0;
	InstanceID = IncrementInstanceID++;

	if (InMontage)
	{
		Montage = InMontage;
		SetPosition(0.f);
		BlendStartAlpha = 0.0f;
		// initialize Blend
		Blend.SetValueRange(0.f, 1.0f);
		RefreshNextPrevSections();

		if (AnimInstance.IsValid() && Montage->CanUseMarkerSync())
		{
			SyncGroupName = Montage->SyncGroup;
		}

		MontageSubStepper.Initialize(*this);
	}
}

void FAnimMontageInstance::RefreshNextPrevSections()
{
	// initialize next section
	if ( Montage->CompositeSections.Num() > 0 )
	{
		NextSections.Empty(Montage->CompositeSections.Num());
		NextSections.AddUninitialized(Montage->CompositeSections.Num());
		PrevSections.Empty(Montage->CompositeSections.Num());
		PrevSections.AddUninitialized(Montage->CompositeSections.Num());

		for (int32 I=0; I<Montage->CompositeSections.Num(); ++I)
		{
			PrevSections[I] = INDEX_NONE;
		}

		for (int32 I=0; I<Montage->CompositeSections.Num(); ++I)
		{
			FCompositeSection & Section = Montage->CompositeSections[I];
			int32 NextSectionIdx = Montage->GetSectionIndex(Section.NextSectionName);
			NextSections[I] = NextSectionIdx;
			if (NextSections.IsValidIndex(NextSectionIdx))
			{
				PrevSections[NextSectionIdx] = I;
			}
		}
	}
}

void FAnimMontageInstance::AddReferencedObjects( FReferenceCollector& Collector )
{
	if (Montage)
	{
		Collector.AddReferencedObject(Montage);
	}
}

void FAnimMontageInstance::Terminate()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimMontageInstance_Terminate);

	if (Montage == nullptr)
	{
		return;
	}

	UAnimMontage* OldMontage = Montage;
	
	if (AnimInstance.IsValid())
	{
		// Must grab a reference on the stack in case "this" is deleted during iteration
		TWeakObjectPtr<UAnimInstance> AnimInstanceLocal = AnimInstance;

		// End all active State BranchingPoints
		for (int32 Index = ActiveStateBranchingPoints.Num() - 1; Index >= 0; Index--)
		{
			FAnimNotifyEvent& NotifyEvent = ActiveStateBranchingPoints[Index];

			if (NotifyEvent.NotifyStateClass)
			{
				FBranchingPointNotifyPayload BranchingPointNotifyPayload(AnimInstance->GetSkelMeshComponent(), Montage, &NotifyEvent, InstanceID, false);
				TRACE_ANIM_NOTIFY(AnimInstance.Get(), NotifyEvent, End);
				NotifyEvent.NotifyStateClass->BranchingPointNotifyEnd(BranchingPointNotifyPayload);

				if (!ValidateInstanceAfterNotifyState(AnimInstanceLocal, NotifyEvent.NotifyStateClass))
				{
					return;
				}
			}
		}
		ActiveStateBranchingPoints.Empty();

		FAnimMontageInstance* SelfPtr = this;
		const uint32 LastMontageFlushFrame = AnimInstanceLocal->GetLastMontageFlushFrame();
		
		// terminating, trigger end
		AnimInstance->QueueMontageEndedEvent(FQueuedMontageEndedEvent(OldMontage, InstanceID, bInterrupted, OnMontageEnded));

		// Safety check if the anim instance is still valid after triggering montage end events 
		if (!AnimInstanceLocal.IsValid())
		{
			return;
		}

		// UninitializeAnimation() empties the MontagesInstances array thus we need to exit early to not use freed memory.
		if (!AnimInstanceLocal->IsInitialized() && MontageCVars::bEarlyOutMontageWhenUninitialized)
		{
			return;
		}

		// Prone to have being free self from trigger montage events.
		if (LastMontageFlushFrame != AnimInstanceLocal->GetLastMontageFlushFrame() && AnimInstanceLocal->MontageInstances.Find(SelfPtr) == INDEX_NONE && MontageCVars::bEarlyOutMontageWhenUninitialized)
		{
			return;
		}
		
		// Clear references to this MontageInstance. Needs to happen before Montage is cleared to nullptr, as TMaps can use that as a key.
		AnimInstance->ClearMontageInstanceReferences(*this);
	}

	// clear Blend curve
	Blend.SetCustomCurve(nullptr);
	Blend.SetBlendOption(EAlphaBlendOption::Linear);

	ActiveBlendProfile = nullptr;
	Montage = nullptr;

	UE_LOGF(LogAnimMontage, Verbose, "Terminating: AnimMontage: %ls", *GetNameSafe(OldMontage));
}

bool FAnimMontageInstance::JumpToSectionName(FName const & SectionName, bool bEndOfSection)
{
	const int32 SectionID = Montage->GetSectionIndex(SectionName);

	if (Montage->IsValidSectionIndex(SectionID))
	{
		FCompositeSection & CurSection = Montage->GetAnimCompositeSection(SectionID);
		const float NewPosition = Montage->CalculatePos(CurSection, bEndOfSection ? Montage->GetSectionLength(SectionID) - UE_KINDA_SMALL_NUMBER : 0.0f);
		SetPosition(NewPosition);
		OnMontagePositionChanged(SectionName);
		
		// changed sections, trigger event
		const bool bLooped = Montage->IsValidSectionIndex(MontageSubStepper.GetCurrentSectionIndex()) ? MontageSubStepper.GetCurrentSectionIndex() == SectionID : false;
		AnimInstance->QueueMontageSectionChangedEvent(FQueuedMontageSectionChangedEvent(Montage, InstanceID, SectionName, bLooped, OnMontageSectionChanged));
		
		return true;
	}

	UE_LOGF(LogAnimMontage, Warning, "JumpToSectionName %ls bEndOfSection: %d failed for Montage %ls",
		*SectionName.ToString(), bEndOfSection, *GetNameSafe(Montage));
	return false;
}

bool FAnimMontageInstance::SetNextSectionName(FName const & SectionName, FName const & NewNextSectionName)
{
	int32 const SectionID = Montage->GetSectionIndex(SectionName);
	int32 const NewNextSectionID = Montage->GetSectionIndex(NewNextSectionName);

	return SetNextSectionID(SectionID, NewNextSectionID);
}

bool FAnimMontageInstance::SetNextSectionID(int32 const & SectionID, int32 const & NewNextSectionID)
{
	bool const bHasValidNextSection = NextSections.IsValidIndex(SectionID);

	// disconnect prev section
	if (bHasValidNextSection && (NextSections[SectionID] != INDEX_NONE) && PrevSections.IsValidIndex(NextSections[SectionID]))
	{
		PrevSections[NextSections[SectionID]] = INDEX_NONE;
	}

	// update in-reverse next section
	if (PrevSections.IsValidIndex(NewNextSectionID))
	{
		PrevSections[NewNextSectionID] = SectionID;
	}

	// update next section for the SectionID
	// NextSection can be invalid
	if (bHasValidNextSection)
	{
		NextSections[SectionID] = NewNextSectionID;
		OnMontagePositionChanged(GetSectionNameFromID(NewNextSectionID));
		return true;
	}

	UE_LOGF(LogAnimMontage, Warning, "SetNextSectionName %ls to %ls failed for Montage %ls",
		*GetSectionNameFromID(SectionID).ToString(), *GetSectionNameFromID(NewNextSectionID).ToString(), *GetNameSafe(Montage));

	return false;
}

void FAnimMontageInstance::OnMontagePositionChanged(FName const & ToSectionName) 
{
	if (bPlaying && IsStopped())
	{
		UE_LOGF(LogAnimMontage, Warning, "Changing section on Montage (%ls) to '%ls' during blend out. This can cause incorrect visuals!",
			*GetNameSafe(Montage), *ToSectionName.ToString());

		Play(PlayRate);
	}
}

FName FAnimMontageInstance::GetCurrentSection() const
{
	if ( Montage )
	{
		float CurrentPosition;
		const int32 CurrentSectionIndex = Montage->GetAnimCompositeSectionIndexFromPos(Position, CurrentPosition);
		if ( Montage->IsValidSectionIndex(CurrentSectionIndex) )
		{
			FCompositeSection& CurrentSection = Montage->GetAnimCompositeSection(CurrentSectionIndex);
			return CurrentSection.SectionName;
		}
	}

	return NAME_None;
}

FName FAnimMontageInstance::GetNextSection() const
{
	if (Montage)
	{
		float CurrentPosition;
		const int32 CurrentSectionIndex = Montage->GetAnimCompositeSectionIndexFromPos(Position, CurrentPosition);
		if (Montage->IsValidSectionIndex(CurrentSectionIndex))
		{
			const int32 NextSectionIndex = GetNextSectionID(CurrentSectionIndex);
			if (Montage->IsValidSectionIndex(NextSectionIndex))
			{
				return GetSectionNameFromID(NextSectionIndex);
			}
		}
	}

	return NAME_None;
}

int32 FAnimMontageInstance::GetNextSectionID(int32 const & CurrentSectionID) const
{
	return NextSections.IsValidIndex(CurrentSectionID) ? NextSections[CurrentSectionID] : INDEX_NONE;
}

FName FAnimMontageInstance::GetSectionNameFromID(int32 const & SectionID) const
{
	if (Montage && Montage->IsValidSectionIndex(SectionID))
	{
		FCompositeSection const & CurrentSection = Montage->GetAnimCompositeSection(SectionID);
		return CurrentSection.SectionName;
	}

	return NAME_None;
}

void FAnimMontageInstance::MontageSync_Follow(struct FAnimMontageInstance* NewLeaderMontageInstance)
{
	// Stop following previous leader if any.
	MontageSync_StopFollowing();

	// Follow new leader
	// Note: we don't really care about detecting loops there, there's no real harm in doing so.
	if (NewLeaderMontageInstance && (NewLeaderMontageInstance != this))
	{
		NewLeaderMontageInstance->MontageSyncFollowers.AddUnique(this);
		MontageSyncLeader = NewLeaderMontageInstance;
	}
}

void FAnimMontageInstance::MontageSync_StopLeading()
{
	for (auto MontageSyncFollower : MontageSyncFollowers)
	{
		if (MontageSyncFollower)
		{
			ensure(MontageSyncFollower->MontageSyncLeader == this);
			MontageSyncFollower->MontageSyncLeader = nullptr;
		}
	}
	MontageSyncFollowers.Empty();
}

void FAnimMontageInstance::MontageSync_StopFollowing()
{
	if (MontageSyncLeader)
	{
		MontageSyncLeader->MontageSyncFollowers.RemoveSingleSwap(this);
		MontageSyncLeader = nullptr;
	}
}

uint32 FAnimMontageInstance::MontageSync_GetFrameCounter() const
{
	return (GFrameCounter % MAX_uint32);
}

bool FAnimMontageInstance::MontageSync_HasBeenUpdatedThisFrame() const
{
	return (MontageSyncUpdateFrameCounter == MontageSync_GetFrameCounter());
}

void FAnimMontageInstance::MontageSync_PreUpdate()
{
	// If we are being synchronized to a leader
	// And our leader HASN'T been updated yet, then we need to synchronize ourselves now.
	// We're basically synchronizing to last frame's values.
	// If we want to avoid that frame of lag, a tick prerequisite should be put between the follower and the leader.
	if (MontageSyncLeader && !MontageSyncLeader->MontageSync_HasBeenUpdatedThisFrame())
	{
		MontageSync_PerformSyncToLeader();
	}
}

void FAnimMontageInstance::MontageSync_PostUpdate()
{
	// Tag ourselves as updated this frame.
	MontageSyncUpdateFrameCounter = MontageSync_GetFrameCounter();

	// If we are being synchronized to a leader
	// And our leader HAS already been updated, then we can synchronize ourselves now.
	// To make sure we are in sync before rendering.
	if (MontageSyncLeader && MontageSyncLeader->MontageSync_HasBeenUpdatedThisFrame())
	{
		MontageSync_PerformSyncToLeader();
	}
}

void FAnimMontageInstance::MontageSync_PerformSyncToLeader()
{
	if (MontageSyncLeader)
	{
		// Sync follower position only if significant error.
		// We don't want continually 'teleport' it, which could have side-effects and skip AnimNotifies.
		const float LeaderPosition = MontageSyncLeader->GetPosition();
		const float FollowerPosition = GetPosition();
		if (FMath::Abs(FollowerPosition - LeaderPosition) > UE_KINDA_SMALL_NUMBER)
		{
			SetPosition(LeaderPosition);
		}

		SetPlayRate(MontageSyncLeader->GetPlayRate());

		// If source and target share same section names, keep them in sync as well. So we properly handle jumps and loops.
		const FName LeaderCurrentSectionName = MontageSyncLeader->GetCurrentSection();
		if ((LeaderCurrentSectionName != NAME_None) && (GetCurrentSection() == LeaderCurrentSectionName))
		{
			const FName LeaderNextSectionName = MontageSyncLeader->GetNextSection();
			SetNextSectionName(LeaderCurrentSectionName, LeaderNextSectionName);
		}
	}
}


void FAnimMontageInstance::UpdateWeight(float DeltaTime)
{
	if ( IsValid() )
	{
		PreviousWeight = Blend.GetBlendedValue();
		const bool bWasComplete = Blend.IsComplete();

		// update weight
		Blend.Update(DeltaTime);

		if (Blend.GetBlendTimeRemaining() < 0.0001f)
		{
			ActiveBlendProfile = nullptr;
		}

		if (!IsStopped() && !bWasComplete && Blend.IsComplete())
		{
			if (UAnimInstance* Inst = AnimInstance.Get())
			{
				Inst->QueueMontageBlendedInEvent(FQueuedMontageBlendedInEvent(Montage, OnMontageBlendedInEnded));
			}
		}

		// Notify weight is max of previous and current as notify could have come
		// from any point between now and last tick
		NotifyWeight = FMath::Max(PreviousWeight, Blend.GetBlendedValue());

		UE_LOGF(LogAnimMontage, Verbose, "UpdateWeight: AnimMontage: %ls,  (DesiredWeight:%0.2f, Weight:%0.2f, PreviousWeight: %0.2f)",
			*Montage->GetName(), GetDesiredWeight(), GetWeight(), PreviousWeight);
		UE_LOGF(LogAnimMontage, Verbose, "Blending Info: BlendOption : %d, AlphaLerp : %0.2f, BlendTime: %0.2f",
			(int32)Blend.GetBlendOption(), Blend.GetAlpha(), Blend.GetBlendTime());
	}
}

bool FAnimMontageInstance::SimulateAdvance(float DeltaTime, float& InOutPosition, FRootMotionMovementParams & OutRootMotionParams) const
{
	if (!IsValid())
	{
		return false;
	}

	const bool bExtractRootMotion = Montage->HasRootMotion() && !IsRootMotionDisabled();

	FMontageSubStepper SimulateMontageSubStepper;
	SimulateMontageSubStepper.Initialize(*this);
	SimulateMontageSubStepper.AddEvaluationTime(DeltaTime);
	while (SimulateMontageSubStepper.HasTimeRemaining())
	{
		const float PreviousSubStepPosition = InOutPosition;
		EMontageSubStepResult SubStepResult = SimulateMontageSubStepper.Advance(InOutPosition, nullptr);

		if (SubStepResult != EMontageSubStepResult::Moved)
		{
			// stop and leave this loop
			break;
		}

		// Extract Root Motion for this time slice, and accumulate it.
		if (bExtractRootMotion)
		{
			OutRootMotionParams.Accumulate(Montage->ExtractRootMotionFromTrackRange(PreviousSubStepPosition, InOutPosition, FAnimExtractContext()));
		}

		// if we reached end of section, and we were not processing a branching point, and no events has messed with out current position..
		// .. Move to next section.
		// (this also handles looping, the same as jumping to a different section).
		if (SimulateMontageSubStepper.HasReachedEndOfSection())
		{
			const int32 CurrentSectionIndex = SimulateMontageSubStepper.GetCurrentSectionIndex();
			const bool bPlayingForward = SimulateMontageSubStepper.GetbPlayingForward();

			// Get recent NextSectionIndex in case it's been changed by previous events.
			const int32 RecentNextSectionIndex = bPlayingForward ? NextSections[CurrentSectionIndex] : PrevSections[CurrentSectionIndex];
			if (RecentNextSectionIndex != INDEX_NONE)
			{
				float LatestNextSectionStartTime;
				float LatestNextSectionEndTime;
				Montage->GetSectionStartAndEndTime(RecentNextSectionIndex, LatestNextSectionStartTime, LatestNextSectionEndTime);

				// Jump to next section's appropriate starting point (start or end).
				InOutPosition = bPlayingForward ? LatestNextSectionStartTime : (LatestNextSectionEndTime - UE_KINDA_SMALL_NUMBER); // remain within section
			}
			else
			{
				// Clamp position to prevent playing past the end of the current section
				float CurrentSectionStartTime, CurrentSectionEndTime;
				Montage->GetSectionStartAndEndTime(CurrentSectionIndex, CurrentSectionStartTime, CurrentSectionEndTime);

				InOutPosition = bPlayingForward ? (CurrentSectionEndTime - UE_KINDA_SMALL_NUMBER) : CurrentSectionStartTime;

				// Reached end of last section. Exit.
				break;
			}
		}
	}

	return true;
}

FSlotAnimationTrack::FSlotAnimationTrack()
	: SlotName(FAnimSlotGroup::DefaultSlotName)
{}

void FMontageSubStepper::Initialize(const struct FAnimMontageInstance& InAnimInstance)
{
	MontageInstance = &InAnimInstance;
	Montage = MontageInstance->Montage;
}

EMontageSubStepResult FMontageSubStepper::Advance(float& InOut_P_Original, const FBranchingPointMarker** OutBranchingPointMarkerPtr)
{
	DeltaMove = 0.f;

	if (MontageInstance == nullptr || (Montage == nullptr))
	{
		return EMontageSubStepResult::InvalidMontage;
	}

	bReachedEndOfSection = false;

	// Update Current Section info in case it's needed by the montage's update loop.
	// We need to do this even if we're not going to move this frame.
	// We could have been moved externally via a SetPosition() call.
	float PositionInSection;
	CurrentSectionIndex = Montage->GetAnimCompositeSectionIndexFromPos(InOut_P_Original, PositionInSection);
	if (!Montage->IsValidSectionIndex(CurrentSectionIndex))
	{
		return EMontageSubStepResult::InvalidSection;
	}

	const FCompositeSection& CurrentSection = Montage->GetAnimCompositeSection(CurrentSectionIndex);
	CurrentSectionStartTime = CurrentSection.GetTime();

	// Find end of current section. We only update one section at a time.
	CurrentSectionLength = Montage->GetSectionLength(CurrentSectionIndex);

	if (!MontageInstance->bPlaying || FMath::IsNearlyZero(TimeRemaining))
	{
		return EMontageSubStepResult::NotMoved;
	}

	// If we're forcing next position, this is our DeltaMove.
	// We don't use play rate and delta time to move.
	if (MontageInstance->ForcedNextToPosition.IsSet())
	{
		const float NewPosition = MontageInstance->ForcedNextToPosition.GetValue();
		if (MontageInstance->ForcedNextFromPosition.IsSet())
		{
			// We are modifying the current position so we also need to update the section and pos in section
			InOut_P_Original = MontageInstance->ForcedNextFromPosition.GetValue();
			CurrentSectionIndex = Montage->GetAnimCompositeSectionIndexFromPos(InOut_P_Original, PositionInSection);
			
			if (!Montage->IsValidSectionIndex(CurrentSectionIndex))
			{
				return EMontageSubStepResult::InvalidSection;
			}
			CurrentSectionStartTime = Montage->GetAnimCompositeSection(CurrentSectionIndex).GetTime();
			CurrentSectionLength = Montage->GetSectionLength(CurrentSectionIndex);
		}

		if (MontageCVars::bUseForcedSequencerMontageLoopBehavior 
			&& MontageInstance->ForcedLoopBoundaryPosition.IsSet() && MontageInstance->ForcedLoopResumePosition.IsSet())
		{
			const float BoundaryPosition = MontageInstance->ForcedLoopBoundaryPosition.GetValue();
			const float ResumePosition = MontageInstance->ForcedLoopResumePosition.GetValue();
			const float TotalDelta = (BoundaryPosition - InOut_P_Original) + (NewPosition - ResumePosition);
			
			DeltaMove = BoundaryPosition - InOut_P_Original;
			PlayRate = TotalDelta / TimeRemaining;
			bPlayingForward = (TotalDelta >= 0.f);
		}
		else
		{
			DeltaMove = NewPosition - InOut_P_Original;
			PlayRate = DeltaMove / TimeRemaining;
			bPlayingForward = (DeltaMove >= 0.f);
		}
		
		TimeStretchMarkerIndex = INDEX_NONE;
	}
	else if (MontageInstance->OnComputeDeltaFrameTime.IsBound())
	{
		// Note that the compute delta frame time ignores play rate, at least for now.
		DeltaMove = MontageInstance->OnComputeDeltaFrameTime.Execute(MontageInstance->GetInstanceID(), InOut_P_Original, TimeRemaining);
		if (FMath::IsNearlyZero(DeltaMove))
		{
			return EMontageSubStepResult::NotMoved;
		}

		// we only set playrate for a sign check later, otherwise unused.
		PlayRate = DeltaMove / TimeRemaining;
		bPlayingForward = (DeltaMove >= 0.f); // *could* go backwards a bit. since music clock is not purely coupled with gameclock. 
		TimeStretchMarkerIndex = INDEX_NONE;
	}
	else
	{
		PlayRate = MontageInstance->PlayRate * Montage->RateScale;

		if (FMath::IsNearlyZero(PlayRate))
		{
			return EMontageSubStepResult::NotMoved;
		}

		// See if we can attempt to use a TimeStretchCurve.
		const bool bAttemptTimeStretchCurve = Montage->TimeStretchCurve.IsValid() && !FMath::IsNearlyEqual(PlayRate, 1.f);
		if (bAttemptTimeStretchCurve)
		{
			// First we need to see if we have valid cached data and if it is up to date.
			ConditionallyUpdateTimeStretchCurveCachedData();
		}

		// If we're not using a TimeStretchCurve, play rate is constant.
		if (!bAttemptTimeStretchCurve || !bHasValidTimeStretchCurveData)
		{
			bPlayingForward = (PlayRate > 0.f);
			DeltaMove = TimeRemaining * PlayRate;
			TimeStretchMarkerIndex = INDEX_NONE;
		}
		else
		{
			// We're using a TimeStretchCurve.

			// Find P_Target for current InOut_P_Original.
			// Not that something external could have modified the montage's position.
			// So we need to refresh our P_Target.
			float P_Target = FindMontagePosition_Target(InOut_P_Original);

			// With P_Target, we're in 'play back time' space. 
			// So we can add our delta time there directly.
			P_Target += bPlayingForward ? TimeRemaining : -TimeRemaining;
			// Make sure we don't exceed our boundaries.
			P_Target = TimeStretchCurveInstance.Clamp_P_Target(P_Target);

			// Now we can map this back into 'original' space and find which frame of animation we should play.
			const float NewP_Original = FindMontagePosition_Original(P_Target);

			// And from there, derive our DeltaMove and actual PlayRate for this substep.
			DeltaMove = NewP_Original - InOut_P_Original;
			PlayRate = DeltaMove / TimeRemaining;
		}
	}

	// Now look for a branching point. If we have one, stop there first to handle it.
	// We need to stop at branching points, because they can trigger events that can cause side effects
	// (jumping to a new position, changing sections, changing play rate, etc).
	if (OutBranchingPointMarkerPtr)
	{
		*OutBranchingPointMarkerPtr = Montage->FindFirstBranchingPointMarker(InOut_P_Original, InOut_P_Original + DeltaMove);
		if (*OutBranchingPointMarkerPtr)
		{
			// If we have a branching point, adjust DeltaMove so we stop there.
			DeltaMove = (*OutBranchingPointMarkerPtr)->TriggerTime - InOut_P_Original;
		}
	}

	// Finally clamp DeltaMove by section markers.
	{
		const float OldDeltaMove = DeltaMove;

		// Clamp DeltaMove based on move allowed within current section
		// We stop at each section marker to evaluate whether we should jump to another section marker or not.
		// Test is inclusive, so we know if we've reached marker or not.
		if (bPlayingForward)
		{
			const float MaxSectionMove = CurrentSectionLength - PositionInSection;
			if (DeltaMove >= MaxSectionMove)
			{
				DeltaMove = MaxSectionMove;
				bReachedEndOfSection = true;
			}
		}
		else
		{
			const float MinSectionMove = /* 0.f */ - PositionInSection;
			if (DeltaMove <= MinSectionMove)
			{
				DeltaMove = MinSectionMove;
				bReachedEndOfSection = true;
			}
		}

		if (OutBranchingPointMarkerPtr && *OutBranchingPointMarkerPtr && (OldDeltaMove != DeltaMove))
		{
			// Clean up the marker since we hit end of a section and overrode the delta move.
			*OutBranchingPointMarkerPtr = nullptr;
		}
	}

	// DeltaMove is now final, see if it has any effect on our position.
	if (FMath::Abs(DeltaMove) > 0.f)
	{
		// Note that we don't worry about looping and wrapping around here.
		// We step per section to simplify code to extract notifies/root motion/etc.
		InOut_P_Original += DeltaMove;

		// Decrease RemainingTime with actual time elapsed 
		// So we can take more substeps as needed.
		const float TimeStep = DeltaMove / PlayRate;
		ensure(TimeStep >= 0.f);
		TimeRemaining = FMath::Max(TimeRemaining - TimeStep, 0.f);

		return EMontageSubStepResult::Moved;
	}
	else
	{
		return EMontageSubStepResult::NotMoved;
	}
}

void FMontageSubStepper::ConditionallyUpdateTimeStretchCurveCachedData()
{
	// CombinedPlayRate defines our overall desired play back time, aka T_Target.
	// When using a TimeStretchCurve, this also defines S and U.
	// Only update these if CombinedPlayRate has changed.
	const float CombinedPlayRate = MontageInstance->PlayRate * Montage->RateScale;
	if (CombinedPlayRate == Cached_CombinedPlayRate)
	{
		return;
	}
	Cached_CombinedPlayRate = CombinedPlayRate;
	
	// We'll set this to true at the end, if we succeed with valid data.
	bHasValidTimeStretchCurveData = false;

	// We should not be using this code path with a 0 play rate
	// or a 1 play rate. we can use traditional cheaper update without curve.
	ensure(!FMath::IsNearlyZero(CombinedPlayRate));
	ensure(!FMath::IsNearlyEqual(CombinedPlayRate, 1.f));

	bPlayingForward = (CombinedPlayRate > 0.f);
	TimeStretchCurveInstance.InitializeFromPlayRate(CombinedPlayRate, Montage->TimeStretchCurve);

	/*
		Section Segment Positions in Target space will have to be re-cached, as needed.
		This is to determine 'remaining time until end' to trigger blend outs.
		But most montages don't use sections.
		So this is optional and done on demand.
	*/
	{
		const int32 NumSections = Montage->CompositeSections.Num();
		SectionStartPositions_Target.Reset(NumSections);
		SectionStartPositions_Target.Init(-1.f, NumSections);
		SectionEndPositions_Target.Reset(NumSections);
		SectionEndPositions_Target.Init(-1.f, NumSections);
	}

	bHasValidTimeStretchCurveData = TimeStretchCurveInstance.HasValidData();
}

float FMontageSubStepper::FindMontagePosition_Target(float In_P_Original)
{
	check(bHasValidTimeStretchCurveData);

	// See if our cached version is not up to date.
	// Then we need to update it.
	if (In_P_Original != Cached_P_Original)
	{
		// Update cached value.
		Cached_P_Original = In_P_Original;

		// Update TimeStretchMarkerIndex if needed.
		// This would happen if we jumped position due to sections or external input.
		TimeStretchCurveInstance.UpdateMarkerIndexForPosition(TimeStretchMarkerIndex, Cached_P_Original, TimeStretchCurveInstance.GetMarkers_Original());

		// With an accurate TimeStretchMarkerIndex, we can map P_Original to P_Target
		Cached_P_Target = TimeStretchCurveInstance.Convert_P_Original_To_Target(TimeStretchMarkerIndex, Cached_P_Original);
	}

	return Cached_P_Target;
}

float FMontageSubStepper::FindMontagePosition_Original(float In_P_Target)
{
	check(bHasValidTimeStretchCurveData);

	// See if our cached version is not up to date.
	// Then we need to update it.
	if (In_P_Target != Cached_P_Target)
	{
		// Update cached value.
		Cached_P_Target = In_P_Target;

		// Update TimeStretchMarkerIndex if needed.
		// This would happen if we jumped position due to sections or external input.
		TimeStretchCurveInstance.UpdateMarkerIndexForPosition(TimeStretchMarkerIndex, Cached_P_Target, TimeStretchCurveInstance.GetMarkers_Target());

		// With an accurate TimeStretchMarkerIndex, we can map P_Original to P_Target
		Cached_P_Original = TimeStretchCurveInstance.Convert_P_Target_To_Original(TimeStretchMarkerIndex, Cached_P_Target);
	}

	return Cached_P_Original;
}

float FMontageSubStepper::GetCurrSectionStartPosition_Target() const
{
	check(bHasValidTimeStretchCurveData);

	const float CachedSectionStartPosition_Target = SectionStartPositions_Target[CurrentSectionIndex];
	if (CachedSectionStartPosition_Target >= 0.f)
	{
		return CachedSectionStartPosition_Target;
	}

	const int32 SectionStartMarkerIndex = TimeStretchCurveInstance.BinarySearchMarkerIndex(CurrentSectionStartTime, TimeStretchCurveInstance.GetMarkers_Original());
	const float SectionStart_Target = TimeStretchCurveInstance.Convert_P_Original_To_Target(SectionStartMarkerIndex, CurrentSectionStartTime);

	SectionStartPositions_Target[CurrentSectionIndex] = SectionStart_Target;

	return SectionStart_Target;
}

float FMontageSubStepper::GetCurrSectionEndPosition_Target() const
{
	check(bHasValidTimeStretchCurveData);

	const float CachedSectionEndPosition_Target = SectionEndPositions_Target[CurrentSectionIndex];
	if (CachedSectionEndPosition_Target >= 0.f)
	{
		return CachedSectionEndPosition_Target;
	}

	const float SectionEnd_Original = CurrentSectionStartTime + CurrentSectionLength;
	const int32 SectionEndMarkerIndex = TimeStretchCurveInstance.BinarySearchMarkerIndex(SectionEnd_Original, TimeStretchCurveInstance.GetMarkers_Original());
	const float SectionEnd_Target = TimeStretchCurveInstance.Convert_P_Original_To_Target(SectionEndMarkerIndex, SectionEnd_Original);

	SectionEndPositions_Target[CurrentSectionIndex] = SectionEnd_Target;

	return SectionEnd_Target;
}

float FMontageSubStepper::GetRemainingPlayTimeToSectionEnd(const float In_P_Original)
{
	// If our current play rate is zero, we can't predict our remaining play time.
	if (FMath::IsNearlyZero(PlayRate))
	{
		return UE_BIG_NUMBER;
	}

	// Find position in montage where current section ends.
	const float CurrSectionEnd_Original = bPlayingForward
		? (CurrentSectionStartTime + CurrentSectionLength)
		: CurrentSectionStartTime;

	// If we have no TimeStretchCurve, it's pretty straight forward.
	// Assume constant play rate.
	if (TimeStretchMarkerIndex == INDEX_NONE)
	{
		const float DeltaPositionToEnd = CurrSectionEnd_Original - In_P_Original;
		const float RemainingPlayTime = FMath::Abs(DeltaPositionToEnd / PlayRate);
		return RemainingPlayTime;
	}

	// We're using a TimeStretchCurve.
	check(bHasValidTimeStretchCurveData);

	// Find our position in 'target' space. This is in play back time.
	const float P_Target = FindMontagePosition_Target(In_P_Original);
	if (bPlayingForward)
	{
		// Find CurrSectionEnd_Target.
		if (FMath::IsNearlyEqual(CurrSectionEnd_Original, TimeStretchCurveInstance.Get_T_Original()))
		{
			const float RemainingPlayTime = (TimeStretchCurveInstance.Get_T_Target() - P_Target);
			return RemainingPlayTime;
		}
		else
		{
			const float CurrSectionEnd_Target = GetCurrSectionEndPosition_Target();
			const float RemainingPlayTime = (CurrSectionEnd_Target - P_Target);
			return RemainingPlayTime;
		}
	}
	// Playing Backwards
	else
	{
		// Find CurrSectionEnd_Target.
		if (FMath::IsNearlyEqual(CurrSectionEnd_Original, 0.f))
		{
			const float RemainingPlayTime = P_Target;
			return RemainingPlayTime;
		}
		else
		{
			const float CurrSectionStart_Target = GetCurrSectionStartPosition_Target();
			const float RemainingPlayTime = (P_Target - CurrSectionStart_Target);
			return RemainingPlayTime;
		}
	}
}

#if WITH_EDITOR
void FAnimMontageInstance::EditorOnly_PreAdvance()
{
	// this is necessary and it is not easy to do outside of here
	// since undo also can change composite sections
	if ((Montage->CompositeSections.Num() != NextSections.Num()) || (Montage->CompositeSections.Num() != PrevSections.Num()))
	{
		RefreshNextPrevSections();
	}

	// Auto refresh this in editor to catch changes being made to AnimNotifies.
	// RefreshCacheData should handle this but I'm not 100% sure it will cover all existing cases
	Montage->RefreshBranchingPointMarkers();

	// Bake TimeStretchCurve in editor to catch any edits made to source curve.
	Montage->BakeTimeStretchCurve();
	// Clear cached data, so it can be recached from updated time stretch curve.
	MontageSubStepper.ClearCachedData();
}
#endif

void FAnimMontageInstance::Advance(float DeltaTime, struct FRootMotionMovementParams* OutRootMotionParams, bool bBlendRootMotion)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimMontageInstance_Advance);
	FScopeCycleCounterUObject MontageScope(Montage);

	if (IsValid())
	{
		// with custom curves, we can't just filter by weight
		// also if you have custom curve with longer 0, you'll likely to pause montage during that blending time
		// I think that is a bug. It still should move, the weight might come back later. 
		if (bPlaying)
		{
			const bool bExtractRootMotion = (OutRootMotionParams != nullptr) && Montage->HasRootMotion();
			
			DeltaTimeRecord.Set(Position, 0.f);

			bDidUseMarkerSyncThisTick = CanUseMarkerSync();
			if (bDidUseMarkerSyncThisTick)
			{
				MarkersPassedThisTick.Reset();
			}
			
			/** 
				Limit number of iterations for performance.
				This can get out of control if PlayRate is set really high, or there is a hitch, and Montage is looping for example.
			*/
			const int32 MaxIterations = 10;
			int32 NumIterations = 0;

			/** 
				If we're hitting our max number of iterations for whatever reason,
				make sure we're not accumulating too much time, and go out of range.
			*/
			if (MontageSubStepper.GetRemainingTime() < 10.f)
			{
				MontageSubStepper.AddEvaluationTime(DeltaTime);
			}

			// Gather active anim state notifies if DeltaTime == 0 (happens when TimeDilation is 0.f), so these are not prematurely ended
			if (DeltaTime == 0.f)
			{
				HandleEvents(Position, Position, nullptr);
			}

			while (bPlaying && MontageSubStepper.HasTimeRemaining() && (++NumIterations < MaxIterations))
			{
				SCOPE_CYCLE_COUNTER(STAT_AnimMontageInstance_Advance_Iteration);

				const float PreviousSubStepPosition = (MontageCVars::bUseForcedFromPositionAsSubStepOrigin && ForcedNextFromPosition.IsSet())
					? ForcedNextFromPosition.GetValue()
					: Position;
				
				const FBranchingPointMarker* BranchingPointMarker = nullptr;
				EMontageSubStepResult SubStepResult = MontageSubStepper.Advance(Position, &BranchingPointMarker);

				if (SubStepResult == EMontageSubStepResult::InvalidSection
					|| SubStepResult == EMontageSubStepResult::InvalidMontage)
				{
					// stop and leave this loop
					Stop(FAlphaBlend(Montage->BlendOut, Montage->BlendOut.GetBlendTime() * DefaultBlendTimeMultiplier), false);
					break;
				}

				const float SubStepDeltaMove = MontageSubStepper.GetDeltaMove();
				DeltaTimeRecord.Delta += SubStepDeltaMove;
				const bool bPlayingForward = MontageSubStepper.GetbPlayingForward();

				// If current section is last one, check to trigger a blend out and if it hasn't stopped yet, see if we should stop
				// We check this even if we haven't moved, in case our position was different from last frame.
				// (Code triggered a position jump).
				if (!IsStopped() && bEnableAutoBlendOut)
				{
					const int32 CurrentSectionIndex = MontageSubStepper.GetCurrentSectionIndex();
					check(NextSections.IsValidIndex(CurrentSectionIndex));
					const int32 NextSectionIndex = bPlayingForward ? NextSections[CurrentSectionIndex] : PrevSections[CurrentSectionIndex];
					if (NextSectionIndex == INDEX_NONE)
					{
						const float PlayTimeToEnd = MontageSubStepper.GetRemainingPlayTimeToSectionEnd(Position);

						const bool bCustomBlendOutTriggerTime = (Montage->BlendOutTriggerTime >= 0);
						const float DefaultBlendOutTime = Montage->BlendOut.GetBlendTime() * DefaultBlendTimeMultiplier;
						const float BlendOutTriggerTime = bCustomBlendOutTriggerTime ? Montage->BlendOutTriggerTime : DefaultBlendOutTime;

						// ... trigger blend out if within blend out time window.
						if (PlayTimeToEnd <= FMath::Max<float>(BlendOutTriggerTime, UE_KINDA_SMALL_NUMBER))
						{
							const float BlendOutTime = bCustomBlendOutTriggerTime ? DefaultBlendOutTime : PlayTimeToEnd;
							Stop(FAlphaBlend(Montage->BlendOut, BlendOutTime), false);
						}
					}
				}

				const bool bHaveMoved = (SubStepResult == EMontageSubStepResult::Moved);
				if (bHaveMoved)
				{
					if (bDidUseMarkerSyncThisTick)
					{
						Montage->MarkerData.CollectMarkersInRange(PreviousSubStepPosition, Position, MarkersPassedThisTick, SubStepDeltaMove, nullptr);
					}

					// Extract Root Motion for this time slice, and accumulate it.
					// IsRootMotionDisabled() can be changed by AnimNotifyState BranchingPoints while advancing, so it needs to be checked here.
					if (bExtractRootMotion && AnimInstance.IsValid() && !IsRootMotionDisabled())
					{
						const FTransform RootMotion = Montage->ExtractRootMotionFromTrackRange(PreviousSubStepPosition, Position, FAnimExtractContext());
						if (bBlendRootMotion)
						{
							// Defer blending in our root motion until after we get our slot weight updated
							const float Weight = Blend.GetBlendedValue();
							AnimInstance.Get()->QueueRootMotionBlend(RootMotion, Montage->SlotAnimTracks[0].SlotName, Weight);
						}
						else
						{
							OutRootMotionParams->Accumulate(RootMotion);
						}

						UE_LOGF(LogRootMotion, Log, "\tFAnimMontageInstance::Advance ExtractedRootMotion: %ls, AccumulatedRootMotion: %ls, bBlendRootMotion: %d"
							, *RootMotion.GetTranslation().ToCompactString()
							, *OutRootMotionParams->GetRootMotionTransform().GetTranslation().ToCompactString()
							, bBlendRootMotion
						);
					}
				}

				// Delegate has to be called last in this loop
				// so that if this changes position, the new position will be applied in the next loop
				// first need to have event handler to handle it
				// Save off position before triggering events, in case they cause a jump to another position
				const float PositionBeforeFiringEvents = Position;

				if(bHaveMoved)
				{
					// Save position before firing events.
					if (!bInterrupted)
					{
						// Must grab a reference on the stack in case "this" is deleted during iteration
						TWeakObjectPtr<UAnimInstance> AnimInstanceLocal = AnimInstance;

						HandleEvents(PreviousSubStepPosition, Position, BranchingPointMarker);

						// Break out if we no longer have active montage instances. This may happen when we call UninitializeAnimation from a notify
						if (AnimInstanceLocal.IsValid() && AnimInstanceLocal->MontageInstances.Num() == 0)
						{
							return;
						}
					}
				}
				
				// After the first loop segment has been processed and events have been handled,
				// clear out the loop data and update the position for the next iteration 
				if (MontageCVars::bUseForcedSequencerMontageLoopBehavior 
					&& ForcedLoopBoundaryPosition.IsSet() && ForcedLoopResumePosition.IsSet())
				{
					Position = ForcedLoopResumePosition.GetValue();
					ForcedNextFromPosition = ForcedLoopResumePosition.GetValue();
					ForcedLoopBoundaryPosition.Reset();
					ForcedLoopResumePosition.Reset();
				}

				// Note that we have to check this even if there is no time remaining, in order to correctly handle loops
				// CVar allows reverting to old behavior, in case a project relies on it
				if (MontageCVars::bEndSectionRequiresTimeRemaining == false || MontageSubStepper.HasTimeRemaining())
				{
					// if we reached end of section, and we were not processing a branching point, and no events has messed with out current position..
					// .. Move to next section.
					// (this also handles looping, the same as jumping to a different section).
					if (MontageSubStepper.HasReachedEndOfSection() && !BranchingPointMarker && (PositionBeforeFiringEvents == Position))
					{
						// Get recent NextSectionIndex in case it's been changed by previous events.
						const int32 CurrentSectionIndex = MontageSubStepper.GetCurrentSectionIndex();
						const int32 RecentNextSectionIndex = bPlayingForward ? NextSections[CurrentSectionIndex] : PrevSections[CurrentSectionIndex];
						const float EndOffset = UE_KINDA_SMALL_NUMBER / 2.f; //KINDA_SMALL_NUMBER/2 because we use KINDA_SMALL_NUMBER to offset notifies for triggering and SMALL_NUMBER is too small

						if (RecentNextSectionIndex != INDEX_NONE)
						{
							float LatestNextSectionStartTime, LatestNextSectionEndTime;
							Montage->GetSectionStartAndEndTime(RecentNextSectionIndex, LatestNextSectionStartTime, LatestNextSectionEndTime);

							// Jump to next section's appropriate starting point (start or end).
							Position = bPlayingForward ? LatestNextSectionStartTime : (LatestNextSectionEndTime - EndOffset);
							SubStepResult = EMontageSubStepResult::Moved;
							
							const bool bLooped = CurrentSectionIndex == RecentNextSectionIndex;
							AnimInstance->QueueMontageSectionChangedEvent(FQueuedMontageSectionChangedEvent(Montage, InstanceID, Montage->GetSectionName(RecentNextSectionIndex), bLooped, OnMontageSectionChanged));
						}
						else
						{
							// If there is no next section and we've reached the end of this one, exit

							// Stop playing and clamp position to prevent playing animation data past the end of the current section
							// We already called Stop above if needed, like if bEnableAutoBlendOut is true
							bPlaying = false;

							float CurrentSectionStartTime, CurrentSectionEndTime;
							Montage->GetSectionStartAndEndTime(CurrentSectionIndex, CurrentSectionStartTime, CurrentSectionEndTime);

							Position = bPlayingForward ? (CurrentSectionEndTime - EndOffset) : CurrentSectionStartTime;
							SubStepResult = EMontageSubStepResult::Moved;

							break;
						}
					}
				}

				if (SubStepResult == EMontageSubStepResult::NotMoved)
				{
					// If the montage has not advanced because ForcedNextFromPosition and ForcedNextToPosition are set to the same value, we need to keep any active notify state alive
					// (This can happen for example in frame-locked Sequencer setups with unstable frame times)
					if (ForcedNextFromPosition.IsSet() && ForcedNextToPosition.IsSet() 
						&& ForcedNextFromPosition.GetValue() == ForcedNextToPosition.GetValue())
					{
						const float ForcedPosition = ForcedNextFromPosition.GetValue();
						HandleEvents(ForcedPosition, ForcedPosition, nullptr);
					}
					
					// If it hasn't moved, there is nothing much to do but weight update
					break;
				}
			}
		
			// if we had a ForcedNextPosition set, reset it.
			ForcedNextToPosition.Reset();
			ForcedNextFromPosition.Reset();
			ForcedLoopBoundaryPosition.Reset();
			ForcedLoopResumePosition.Reset();
		}
	}

#if ANIM_TRACE_ENABLED
	for(const FPassedMarker& PassedMarker : MarkersPassedThisTick)
	{
		TRACE_ANIM_SYNC_MARKER(AnimInstance.Get(), PassedMarker);
	}
#endif

	// If this Montage has no weight, it should be terminated.
	if (IsStopped() && (Blend.IsComplete()))
	{
		// nothing else to do
		Terminate();
		return;
	}

	if (!bInterrupted && AnimInstance.IsValid())
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimMontageInstance_TickBranchPoints);

		// Must grab a reference on the stack in case "this" is deleted during iteration
		TWeakObjectPtr<UAnimInstance> AnimInstanceLocal = AnimInstance;

		// Tick all active state branching points
		for (int32 Index = 0; Index < ActiveStateBranchingPoints.Num(); Index++)
		{
			FAnimNotifyEvent& NotifyEvent = ActiveStateBranchingPoints[Index];
			if (NotifyEvent.NotifyStateClass)
			{
				FBranchingPointNotifyPayload BranchingPointNotifyPayload(AnimInstance->GetSkelMeshComponent(), Montage, &NotifyEvent, InstanceID);
				NotifyEvent.NotifyStateClass->BranchingPointNotifyTick(BranchingPointNotifyPayload, DeltaTime);

				// Break out if we no longer have active montage instances. This may happen when we call UninitializeAnimation from a notify
				if (!ValidateInstanceAfterNotifyState(AnimInstanceLocal, NotifyEvent.NotifyStateClass))
				{
					return;
				}
			}
		}
	}
}

void FAnimMontageInstance::HandleEvents(float PreviousTrackPos, float CurrentTrackPos, const FBranchingPointMarker* BranchingPointMarker)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimMontageInstance_HandleEvents);

	// Skip notifies and branching points if montage has been interrupted.
	if (bInterrupted)
	{
		return;
	}

	// Now get active Notifies based on how it advanced
	if (AnimInstance.IsValid())
	{
		FAnimTickRecord TickRecord;

		// Used to ensure all gathered notifies know the current montage time at the point they were queued.
		TickRecord.TimeAccumulator = &CurrentTrackPos;
		
		// Add instance ID to context to differentiate notifies between different instances of the same montage
		TickRecord.MakeContextData<UE::Anim::FAnimNotifyMontageInstanceContext>(InstanceID);

		FAnimNotifyContext NotifyContext(TickRecord);

		// Queue all notifies fired from the AnimMontage's Notify Track.
		{
			// We already break up AnimMontage update to handle looping, so we guarantee that PreviousPos and CurrentPos are contiguous.
			Montage->GetAnimNotifiesFromDeltaPositions(PreviousTrackPos, CurrentTrackPos, NotifyContext);

			// For Montage only, remove notifies marked as 'branching points'. They are not queued and are handled separately.
			Montage->FilterOutNotifyBranchingPoints(NotifyContext.ActiveNotifies);

			// Queue active non-'branching point' notifies.
			AnimInstance->NotifyQueue.AddAnimNotifies(NotifyContext.ActiveNotifies, NotifyWeight);
		}

		// Queue all notifies fired by all the animations within the AnimMontage. We'll do this for all slot tracks.
		{
			TMap<FName, TArray<FAnimNotifyEventReference>> NotifyMap;
			
			for (auto SlotTrack = Montage->SlotAnimTracks.CreateIterator(); SlotTrack; ++SlotTrack)
			{
				TArray<FAnimNotifyEventReference>& CurrentSlotNotifies = NotifyMap.FindOrAdd(SlotTrack->SlotName);

				// Queue active notifies from current slot.
				{
					NotifyContext.ActiveNotifies.Reset();
					SlotTrack->AnimTrack.GetAnimNotifiesFromTrackPositions(PreviousTrackPos, CurrentTrackPos, NotifyContext);
					Swap(CurrentSlotNotifies, NotifyContext.ActiveNotifies);
				}
			}

			// Queue active unfiltered notifies from slot tracks.
			AnimInstance->NotifyQueue.AddAnimNotifies(NotifyMap, NotifyWeight);	
		}
	}

	// Update active state branching points, before we handle the immediate tick marker.
	// In case our position jumped on the timeline, we need to begin/end state branching points accordingly.
	// If this fails, this montage instance is no longer valid. Return to avoid crash.
	if (!UpdateActiveStateBranchingPoints(CurrentTrackPos))
	{
		return;
	}

	// Trigger ImmediateTickMarker event if we have one
	if (BranchingPointMarker)
	{
		BranchingPointEventHandler(BranchingPointMarker);
	}
}

bool FAnimMontageInstance::UpdateActiveStateBranchingPoints(float CurrentTrackPosition)
{
	int32 NumStateBranchingPoints = Montage->BranchingPointStateNotifyIndices.Num();

	if (AnimInstance.IsValid() && NumStateBranchingPoints > 0)
	{
		// Must grab a reference on the stack in case "this" is deleted during iteration
		TWeakObjectPtr<UAnimInstance> AnimInstanceLocal = AnimInstance;

		// End no longer active events first. We want this to happen before we trigger NotifyBegin on newly active events.
		for (int32 Index = ActiveStateBranchingPoints.Num() - 1; Index >= 0; Index--)
		{
			FAnimNotifyEvent& NotifyEvent = ActiveStateBranchingPoints[Index];

			if (NotifyEvent.NotifyStateClass)
			{
				const float NotifyStartTime = NotifyEvent.GetTriggerTime();
				const float NotifyEndTime = NotifyEvent.GetEndTriggerTime();
				bool bNotifyIsActive = (CurrentTrackPosition > NotifyStartTime) && (CurrentTrackPosition <= NotifyEndTime);

				if (!bNotifyIsActive)
				{
					FBranchingPointNotifyPayload BranchingPointNotifyPayload(AnimInstance->GetSkelMeshComponent(), Montage, &NotifyEvent, InstanceID, true);
					TRACE_ANIM_NOTIFY(AnimInstance.Get(), NotifyEvent, End);
					NotifyEvent.NotifyStateClass->BranchingPointNotifyEnd(BranchingPointNotifyPayload);

					// Break out if we no longer have active montage instances. This may happen when we call UninitializeAnimation from a notify
					if (!ValidateInstanceAfterNotifyState(AnimInstanceLocal, NotifyEvent.NotifyStateClass))
					{
						return false;
					}

					ActiveStateBranchingPoints.RemoveAt(Index, 1);
				}
			}
		}

		// Then, begin newly active notifies
		for (int32 Index = 0; Index < NumStateBranchingPoints; Index++)
		{
			const int32 NotifyIndex = Montage->BranchingPointStateNotifyIndices[Index];
			FAnimNotifyEvent& NotifyEvent = Montage->Notifies[NotifyIndex];

			if (NotifyEvent.NotifyStateClass)
			{
				const float NotifyStartTime = NotifyEvent.GetTriggerTime();
				const float NotifyEndTime = NotifyEvent.GetEndTriggerTime();

				bool bNotifyIsActive = (CurrentTrackPosition > NotifyStartTime) && (CurrentTrackPosition <= NotifyEndTime);
				if (bNotifyIsActive && !ActiveStateBranchingPoints.Contains(NotifyEvent))
				{
					FBranchingPointNotifyPayload BranchingPointNotifyPayload(AnimInstance->GetSkelMeshComponent(), Montage, &NotifyEvent, InstanceID);
					TRACE_ANIM_NOTIFY(AnimInstance.Get(), NotifyEvent, Begin);
					NotifyEvent.NotifyStateClass->BranchingPointNotifyBegin(BranchingPointNotifyPayload);

					// Break out if we no longer have active montage instances. This may happen when we call UninitializeAnimation from a notify
					if (!ValidateInstanceAfterNotifyState(AnimInstanceLocal, NotifyEvent.NotifyStateClass))
					{
						return false;
					}

					ActiveStateBranchingPoints.Add(NotifyEvent);
				}
			}
		}
	}

	return true;
}

void FAnimMontageInstance::BranchingPointEventHandler(const FBranchingPointMarker* BranchingPointMarker)
{
	if (AnimInstance.IsValid() && Montage && BranchingPointMarker)
	{
		// Must grab a reference on the stack in case "this" is deleted during iteration
		TWeakObjectPtr<UAnimInstance> AnimInstanceLocal = AnimInstance;

		FAnimNotifyEvent* NotifyEvent = (BranchingPointMarker->NotifyIndex < Montage->Notifies.Num()) ? &Montage->Notifies[BranchingPointMarker->NotifyIndex] : nullptr;
		if (NotifyEvent)
		{
			// Handle backwards compatibility with older BranchingPoints.
			if (NotifyEvent->bConvertedFromBranchingPoint && (NotifyEvent->NotifyName != NAME_None))
			{
				FString FuncName = FString::Printf(TEXT("MontageBranchingPoint_%s"), *NotifyEvent->NotifyName.ToString());
				FName FuncFName = FName(*FuncName);

				UFunction* Function = AnimInstance.Get()->FindFunction(FuncFName);
				if (Function)
				{
					AnimInstance.Get()->ProcessEvent(Function, nullptr);
				}
				// In case older BranchingPoint has been re-implemented as a new Custom Notify, this is if BranchingPoint function hasn't been found.
				else
				{
					AnimInstance.Get()->TriggerSingleAnimNotify(NotifyEvent);
				}
			}
			else if (NotifyEvent->NotifyStateClass != nullptr)
			{
				if (BranchingPointMarker->NotifyEventType == EAnimNotifyEventType::Begin)
				{
					FBranchingPointNotifyPayload BranchingPointNotifyPayload(AnimInstance->GetSkelMeshComponent(), Montage, NotifyEvent, InstanceID);
					TRACE_ANIM_NOTIFY(AnimInstance.Get(), *NotifyEvent, Begin);
					NotifyEvent->NotifyStateClass->BranchingPointNotifyBegin(BranchingPointNotifyPayload);

					if (!ValidateInstanceAfterNotifyState(AnimInstanceLocal, NotifyEvent->NotifyStateClass))
					{
						return;
					}

					ActiveStateBranchingPoints.Add(*NotifyEvent);
				}
				else
				{
					FBranchingPointNotifyPayload BranchingPointNotifyPayload(AnimInstance->GetSkelMeshComponent(), Montage, NotifyEvent, InstanceID, true);
					TRACE_ANIM_NOTIFY(AnimInstance.Get(), *NotifyEvent, End);
					NotifyEvent->NotifyStateClass->BranchingPointNotifyEnd(BranchingPointNotifyPayload);

					if (!ValidateInstanceAfterNotifyState(AnimInstanceLocal, NotifyEvent->NotifyStateClass))
					{
						return;
					}

					ActiveStateBranchingPoints.RemoveSingleSwap(*NotifyEvent);
				}
			}
			// Non state notify with a native notify class
			else if	(NotifyEvent->Notify != nullptr)
			{
				// Implemented notify: just call Notify. UAnimNotify will forward this to the event which will do the work.
				FBranchingPointNotifyPayload BranchingPointNotifyPayload(AnimInstance->GetSkelMeshComponent(), Montage, NotifyEvent, InstanceID);
				TRACE_ANIM_NOTIFY(AnimInstance.Get(), *NotifyEvent, Event);
				NotifyEvent->Notify->BranchingPointNotify(BranchingPointNotifyPayload);
			}
			// Try to match a notify function by name.
			else
			{
				AnimInstance.Get()->TriggerSingleAnimNotify(NotifyEvent);
			}
		}
	}
}

UAnimMontage* FAnimMontageInstance::PreviewSequencerMontagePosition(FName SlotName, USkeletalMeshComponent* SkeletalMeshComponent, int32& InOutInstanceId, UAnimSequenceBase* InAnimSequence, float InFromPosition, float InToPosition, float Weight, bool bLooping, bool bFireNotifies, bool bPlaying)
{
	if (SkeletalMeshComponent)
	{
		return PreviewSequencerMontagePosition(SlotName, SkeletalMeshComponent, SkeletalMeshComponent->GetAnimInstance(), InOutInstanceId, InAnimSequence, InFromPosition, InToPosition, Weight, bLooping, bFireNotifies, bPlaying);
	}

	return nullptr;
}

UAnimMontage* FAnimMontageInstance::SetSequencerMontagePosition(FName SlotName, USkeletalMeshComponent* SkeletalMeshComponent, int32& InOutInstanceId, UAnimSequenceBase* InAnimSequence, float InFromPosition, float InToPosition, float Weight, bool bLooping, bool bPlaying)
{
	if (SkeletalMeshComponent)
	{
		return SetSequencerMontagePosition(SlotName, SkeletalMeshComponent->GetAnimInstance(), InOutInstanceId, InAnimSequence, InFromPosition, InToPosition, Weight, bLooping, bPlaying);
	}

	return nullptr;
}

UAnimMontage* FAnimMontageInstance::SetSequencerMontagePosition(FName SlotName, UAnimInstance* AnimInstance, int32& InOutInstanceId, UAnimSequenceBase* InAnimSequence, float InFromPosition, float InToPosition, float Weight, bool bLooping, bool bInPlaying)
{
	UAnimInstance* AnimInst = AnimInstance;
	if (AnimInst)
	{
		UAnimMontage* PlayingMontage = nullptr;
		FAnimMontageInstance* MontageInstanceToUpdate = AnimInst->GetMontageInstanceForID(InOutInstanceId);

		if (!MontageInstanceToUpdate)
		{
			PlayingMontage = UAnimMontage::CreateSlotAnimationAsDynamicMontage(InAnimSequence, SlotName, 0.0f, 0.0f, 0.f, 1);
			if (PlayingMontage)
			{
				AnimInst->Montage_Play(PlayingMontage, 1.f, EMontagePlayReturnType::MontageLength, 0.f, false);
				MontageInstanceToUpdate = AnimInst->GetActiveInstanceForMontage(PlayingMontage);
				// this is sequencer set up, we disable auto blend out
				if (MontageInstanceToUpdate)
				{
					MontageInstanceToUpdate->bEnableAutoBlendOut = false;
				}
			}
		}

		if (MontageInstanceToUpdate)
		{
			PlayingMontage = MontageInstanceToUpdate->Montage;
			InOutInstanceId = MontageInstanceToUpdate->GetInstanceID();

			// ensure full weighting to this instance
			MontageInstanceToUpdate->Blend.SetDesiredValue(Weight);
			MontageInstanceToUpdate->Blend.SetAlpha(Weight);
			MontageInstanceToUpdate->BlendStartAlpha = MontageInstanceToUpdate->Blend.GetAlpha();
			
			if (bInPlaying)
			{
				// If the forced from position is still set, it means we either haven't advanced the montage yet (e.g. because of URO) 
				// or the sequencer is looping and calling this function twice in one frame before the montage had a chance to advance 
				if (MontageCVars::bUseForcedSequencerMontageLoopBehavior 
					&& MontageInstanceToUpdate->ForcedNextFromPosition.IsSet() && MontageInstanceToUpdate->ForcedNextToPosition.IsSet())
				{
					const float ForcedFromPosition = MontageInstanceToUpdate->ForcedNextFromPosition.GetValue();
					const bool bPlayingForwards = InToPosition >= InFromPosition; 
					const bool bIsLooping = bPlayingForwards ? (ForcedFromPosition > InToPosition) : (ForcedFromPosition < InToPosition);

					// If we detected that a loop is being performed, cache the loop boundary positions 
					// so we can evaluate them correctly when advancing the montage 
					if (bIsLooping)
					{
						// This is the location in the animation that marks the end of that section 
						MontageInstanceToUpdate->ForcedLoopBoundaryPosition = MontageInstanceToUpdate->ForcedNextToPosition.GetValue();
					
						// This is the location we want to resume from after hitting the loop boundary 
						MontageInstanceToUpdate->ForcedLoopResumePosition = InFromPosition;
					}
					
					// Update the from position to the previous from position to account for the full range of update that needs to happen
					// Even if no loop is detected we want to update the from position to allow full range evaluation if sequencer happened to tick
					// more frequently than the montage advanced (e.g. if URO is active)
					InFromPosition = ForcedFromPosition;
				}
				
				MontageInstanceToUpdate->SetNextPositionWithEvents(InFromPosition, InToPosition);
			}
			else
			{
				MontageInstanceToUpdate->SetPosition(InToPosition);
			}

			MontageInstanceToUpdate->bPlaying = bInPlaying;
			return PlayingMontage;
		}
	}
	else
	{
		UE_LOGF(LogSkeletalMesh, Warning, "Invalid animation configuration when attempting to set animation possition with : %ls", *InAnimSequence->GetName());
	}

	return nullptr;
}

UAnimMontage* FAnimMontageInstance::PreviewSequencerMontagePosition(FName SlotName, USkeletalMeshComponent* SkeletalMeshComponent, UAnimInstance* AnimInstance, int32& InOutInstanceId, UAnimSequenceBase* InAnimSequence, float InFromPosition, float InToPosition, float Weight, bool bLooping, bool bFireNotifies, bool bInPlaying)
{
	UAnimInstance* AnimInst = AnimInstance;
	if (AnimInst)
	{
		FAnimMontageInstance* MontageInstanceToUpdate = AnimInst->GetMontageInstanceForID(InOutInstanceId);

		UAnimMontage* PlayingMontage = SetSequencerMontagePosition(SlotName, AnimInst, InOutInstanceId, InAnimSequence, InFromPosition, InToPosition, Weight, bLooping, bInPlaying);
		if (PlayingMontage)
		{
			// we have to get it again in case if this is new
			MontageInstanceToUpdate = AnimInst->GetMontageInstanceForID(InOutInstanceId);
			// since we don't advance montage in the tick, we manually have to handle notifies
			MontageInstanceToUpdate->HandleEvents(InFromPosition, InToPosition, nullptr);
			if (!bFireNotifies)
			{
				AnimInst->NotifyQueue.Reset(SkeletalMeshComponent);
			}

			return PlayingMontage;
		}
	}

	return nullptr;
}

UAnimMontage* UAnimMontage::CreateSlotAnimationAsDynamicMontage(UAnimSequenceBase* Asset, FName SlotNodeName, float BlendInTime, float BlendOutTime, float InPlayRate, int32 LoopCount, float BlendOutTriggerTime, float InTimeToStartMontageAt)
{
	FMontageBlendSettings BlendInSettings(BlendInTime);
	FMontageBlendSettings BlendOutSettings(BlendOutTime);

	// InTimeToStartMontageAt is an unused argument. Keeping it to avoid changing public api.
	return CreateSlotAnimationAsDynamicMontage_WithBlendSettings(Asset, SlotNodeName, BlendInSettings, BlendOutSettings, InPlayRate, LoopCount, BlendOutTriggerTime);
}

UAnimMontage* UAnimMontage::CreateSlotAnimationAsDynamicMontage_WithFractionalLoops(UAnimSequenceBase* Asset, FName SlotNodeName, float BlendInTime, float BlendOutTime, float LoopCount, float BlendOutTriggerTime)
{
	if (!ensure(LoopCount > 0.0))
	{
		return nullptr;
	}

	UAnimMontage* BaseMontage = CreateSlotAnimationAsDynamicMontage(Asset, SlotNodeName, BlendInTime, BlendOutTime, 1.0f, 1, BlendOutTriggerTime);
	if (BaseMontage == nullptr)
	{
		return nullptr;
	}

	if (LoopCount == 1.0)
	{
		return BaseMontage;
	}

	FSlotAnimationTrack& SlotTrack = BaseMontage->SlotAnimTracks[0];
	FAnimSegment& OriginalSegment = SlotTrack.AnimTrack.AnimSegments[0];

	FCompositeSection& Section = BaseMontage->CompositeSections[0];
	Section.Link(Asset, 0.0);

	if (LoopCount < 1.0)
	{
		// Need to shorten the existing segment
		OriginalSegment.AnimEndTime = Asset->GetPlayLength() * LoopCount;
	}
	else
	{
		int32 NumFullLoops = FMath::IsFinite(LoopCount) ? static_cast<int32>(LoopCount) : MAX_int32;
		float FractionalLoop = LoopCount - NumFullLoops;

		// Need to lengthen the existing segment
		OriginalSegment.LoopingCount = NumFullLoops;

		// Need to add a new fractional segment onto the end of the montage
		FAnimSegment& NewSegment = SlotTrack.AnimTrack.AnimSegments.AddDefaulted_GetRef();
		NewSegment.SetAnimReference(Asset, true);

		NewSegment.StartPos = Asset->GetPlayLength() * NumFullLoops;
		NewSegment.AnimEndTime = Asset->GetPlayLength() * FractionalLoop;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BaseMontage->SequenceLength = Asset->GetPlayLength() * LoopCount;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return BaseMontage;
}

UAnimMontage* UAnimMontage::CreateSlotAnimationAsDynamicMontage_WithBlendSettings(UAnimSequenceBase* Asset, FName SlotNodeName, const FMontageBlendSettings& BlendInSettings, const FMontageBlendSettings& BlendOutSettings, float InPlayRate, int32 LoopCount, float InBlendOutTriggerTime)
{
	// create temporary montage and play
	bool bValidAsset = Asset && !Asset->IsA(UAnimMontage::StaticClass());
	if (!bValidAsset)
	{
		// user warning
		UE_LOGF(LogAnimMontage, Warning, "PlaySlotAnimationAsDynamicMontage: Invalid input asset(%ls). If Montage, please use Montage_Play", *GetNameSafe(Asset));
		return nullptr;
	}

	if (SlotNodeName == NAME_None)
	{
		// user warning
		UE_LOGF(LogAnimMontage, Warning, "SlotNode Name is required. Make sure to add Slot Node in your anim graph and name it.");
		return nullptr;
	}

	USkeleton* AssetSkeleton = Asset->GetSkeleton();
	if (!Asset->CanBeUsedInComposition())
	{
		UE_LOGF(LogAnimMontage, Warning, "This animation isn't supported to play as montage");
		return nullptr;
	}

	// now play
	UAnimMontage* NewMontage = NewObject<UAnimMontage>();
	NewMontage->SetSkeleton(AssetSkeleton);

	// add new track
	FSlotAnimationTrack& NewTrack = NewMontage->SlotAnimTracks[0];
	NewTrack.SlotName = SlotNodeName;
	FAnimSegment NewSegment;
	NewSegment.SetAnimReference(Asset, true);
	NewSegment.LoopingCount = LoopCount;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    NewMontage->SequenceLength = NewSegment.GetLength();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	NewTrack.AnimTrack.AnimSegments.Add(NewSegment);

	FCompositeSection NewSection;
	NewSection.SectionName = TEXT("Default");
	NewSection.Link(Asset, Asset->GetPlayLength());
	NewSection.SetTime(0.0f);

	// add new section
	NewMontage->CompositeSections.Add(NewSection);

	NewMontage->BlendIn = FAlphaBlend(BlendInSettings.Blend);
	NewMontage->BlendModeIn = BlendInSettings.BlendMode;
	NewMontage->BlendProfileIn = BlendInSettings.BlendProfile;

	NewMontage->BlendOut = FAlphaBlend(BlendOutSettings.Blend);
	NewMontage->BlendModeOut = BlendOutSettings.BlendMode;
	NewMontage->BlendProfileOut = BlendOutSettings.BlendProfile;

	NewMontage->BlendOutTriggerTime = InBlendOutTriggerTime;

	NewMontage->CommonTargetFrameRate = Asset->GetSamplingFrameRate();
	
	return NewMontage;
}

bool FAnimMontageInstance::CanUseMarkerSync() const
{
	// for now we only allow non-full weight and when blending out
	return SyncGroupName != NAME_None && IsStopped() && Blend.IsComplete() == false;
}

#if WITH_EDITOR
void UAnimMontage::BakeTimeStretchCurve()
{
	TimeStretchCurve.Reset();

	// See if Montage is hosting a curve named 'TimeStretchCurveName'
	const FFloatCurve* TimeStretchFloatCurve = nullptr;
	if (ShouldDataModelBeValid())
	{		
		TimeStretchFloatCurve = GetDataModel()->FindFloatCurve(FAnimationCurveIdentifier(TimeStretchCurveName, ERawCurveTrackTypes::RCT_Float));
	}

	if (TimeStretchFloatCurve == nullptr)
	{
		return;
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TimeStretchCurve.BakeFromFloatCurve(*TimeStretchFloatCurve, SequenceLength);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAnimMontage::PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> ExistingDataModel)
{
	Super::PopulateWithExistingModel(ExistingDataModel);
	
	// Set composite length while model is being populated
	const float CurrentCalculatedLength = CalculateSequenceLength();
	SetCompositeLength(CurrentCalculatedLength);
}

void UAnimMontage::UpdateCommonTargetFrameRate()
{
	CommonTargetFrameRate = FFrameRate(0,0);
	FFrameRate TargetRate = UAnimationSettings::Get()->GetDefaultFrameRate();

	bool bValidFrameRate = true;
	bool bFirst = true;
	for (const FSlotAnimationTrack& Track : SlotAnimTracks)
	{
		for (const FAnimSegment& Segment : Track.AnimTrack.AnimSegments)
		{
			const UAnimSequenceBase* Base = Segment.GetAnimReference();
			if (Base && Base != this)
			{
				const FFrameRate BaseFrameRate = Base->GetSamplingFrameRate();
				if (bFirst)
				{
					TargetRate = BaseFrameRate;
					bFirst = false;
				}
				else
				{
					if (BaseFrameRate.IsValid())
					{
						if (TargetRate.IsMultipleOf(BaseFrameRate))
						{
							TargetRate = BaseFrameRate;
						}
						else if (TargetRate != BaseFrameRate && !BaseFrameRate.IsMultipleOf(TargetRate))
						{
							FString AssetString;
							TArray<UAnimationAsset*> Assets;
							if(GetAllAnimationSequencesReferred(Assets, false))
							{
								for (const UAnimationAsset* AnimAsset : Assets)
								{
									if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(AnimAsset))
									{
										AssetString.Append(FString::Printf(TEXT("\n\t%s - %s"), *AnimSequenceBase->GetName(), *AnimSequenceBase->GetSamplingFrameRate().ToPrettyText().ToString()));
									}
								}
							}						

							if (UE::Anim::CVarOutputMontageFrameRateWarning.GetValueOnAnyThread() == true)
							{
								UE_LOGF(LogAnimation, Warning, "Frame rate of animation %ls (%ls) is incompatible with other animations in Animation Montage %ls - underlying frame-rate will be set to %ls:%ls", *Base->GetName(), *BaseFrameRate.ToPrettyText().ToString(), *GetName(), *Super::GetSamplingFrameRate().ToPrettyText().ToString(), *AssetString);
							}
						
							bValidFrameRate = false;
							break;
						}
					}
					else
					{
						UE_LOGF(LogAnimMontage, Warning, "Invalid frame rate %ls for %ls in %ls", *BaseFrameRate.ToPrettyText().ToString(), *Base->GetName(), *GetName());
					}
				}			
			}	
		}			
	}

	if (bValidFrameRate)
	{
		CommonTargetFrameRate = TargetRate;
	}
}

#endif // WITH_EDITOR

FMontageBlendSettings::FMontageBlendSettings()
	: BlendProfile(nullptr)
	, BlendMode(EMontageBlendMode::Standard)
{}

FMontageBlendSettings::FMontageBlendSettings(float BlendTime)
	: BlendProfile(nullptr)
	, Blend(BlendTime)
	, BlendMode(EMontageBlendMode::Standard)
{}

FMontageBlendSettings::FMontageBlendSettings(const FAlphaBlendArgs& BlendArgs)
	: BlendProfile(nullptr)
	, Blend(BlendArgs)
	, BlendMode(EMontageBlendMode::Standard)
{}

