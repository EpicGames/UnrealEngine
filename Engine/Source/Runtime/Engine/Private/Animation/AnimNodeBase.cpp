// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNodeBase.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimAttributes.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimSubsystem_Base.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNodeBase)

namespace UE::Anim::Private
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Don't inline this function to keep the stack usage down
	FORCENOINLINE void ValidatePose(const FCompactPose& Pose, const FAnimInstanceProxy* AnimInstanceProxy, const FAnimNode_Base* LinkedNode)
	{
		if (Pose.ContainsNaN())
		{
			// Show bone transform with some useful debug info
			const auto& Bones = Pose.GetBones();
			for (int32 CPIndex = 0; CPIndex < Bones.Num(); ++CPIndex)
			{
				const FTransform& Bone = Bones[CPIndex];
				if (Bone.ContainsNaN())
				{
					const FBoneContainer& BoneContainer = Pose.GetBoneContainer();
					const FReferenceSkeleton& RefSkel = BoneContainer.GetReferenceSkeleton();
					const FMeshPoseBoneIndex MeshBoneIndex = BoneContainer.MakeMeshPoseIndex(FCompactPoseBoneIndex(CPIndex));
					ensureMsgf(!Bone.ContainsNaN(), TEXT("Bone (%s) contains NaN from AnimInstance:[%s] Node:[%s] Value:[%s]"),
						*RefSkel.GetBoneName(MeshBoneIndex.GetInt()).ToString(),
						*AnimInstanceProxy->GetAnimInstanceName(), LinkedNode ? *LinkedNode->StaticStruct()->GetName() : TEXT("NULL"),
						*Bone.ToString());
				}
			}
		}

		if (!Pose.IsNormalized())
		{
			// Show bone transform with some useful debug info
			const auto& Bones = Pose.GetBones();
			for (int32 CPIndex = 0; CPIndex < Bones.Num(); ++CPIndex)
			{
				const FTransform& Bone = Bones[CPIndex];
				if (!Bone.IsRotationNormalized())
				{
					const FBoneContainer& BoneContainer = Pose.GetBoneContainer();
					const FReferenceSkeleton& RefSkel = BoneContainer.GetReferenceSkeleton();
					const FMeshPoseBoneIndex MeshBoneIndex = BoneContainer.MakeMeshPoseIndex(FCompactPoseBoneIndex(CPIndex));
					ensureMsgf(Bone.IsRotationNormalized(), TEXT("Bone (%s) Rotation not normalized from AnimInstance:[%s] Node:[%s] Rotation:[%s]"),
						*RefSkel.GetBoneName(MeshBoneIndex.GetInt()).ToString(),
						*AnimInstanceProxy->GetAnimInstanceName(), LinkedNode ? *LinkedNode->StaticStruct()->GetName() : TEXT("NULL"),
						*Bone.GetRotation().ToString());
				}
			}
		}
	}
#endif
}

/////////////////////////////////////////////////////
// FAnimationBaseContext

FAnimationBaseContext::FAnimationBaseContext()
	: AnimInstanceProxy(nullptr)
	, SharedContext(nullptr)
	, CurrentNodeId(INDEX_NONE)
	, PreviousNodeId(INDEX_NONE)
{
}

FAnimationBaseContext::FAnimationBaseContext(FAnimInstanceProxy* InAnimInstanceProxy, FAnimationUpdateSharedContext* InSharedContext)
	: AnimInstanceProxy(InAnimInstanceProxy)
	, SharedContext(InSharedContext)
	, CurrentNodeId(INDEX_NONE)
	, PreviousNodeId(INDEX_NONE)
{
}

IAnimClassInterface* FAnimationBaseContext::GetAnimClass() const
{
	return AnimInstanceProxy ? AnimInstanceProxy->GetAnimClassInterface() : nullptr;
}

UObject* FAnimationBaseContext::GetAnimInstanceObject() const 
{
	return AnimInstanceProxy ? AnimInstanceProxy->GetAnimInstanceObject() : nullptr;
}

#if WITH_EDITORONLY_DATA
UAnimBlueprint* FAnimationBaseContext::GetAnimBlueprint() const
{
	return AnimInstanceProxy ? AnimInstanceProxy->GetAnimBlueprint() : nullptr;
}
#endif //WITH_EDITORONLY_DATA

void FAnimationBaseContext::LogMessageInternal(FName InLogType, const TSharedRef<FTokenizedMessage>& InMessage) const
{
	AnimInstanceProxy->LogMessage(InLogType, InMessage);
}
/////////////////////////////////////////////////////
// FPoseContext

void FPoseContext::InitializeImpl(FAnimInstanceProxy* InAnimInstanceProxy)
{
	AnimInstanceProxy = InAnimInstanceProxy;

	checkSlow(AnimInstanceProxy && AnimInstanceProxy->GetRequiredBones().IsValid());
	const FBoneContainer& RequiredBone = AnimInstanceProxy->GetRequiredBones();
	Pose.SetBoneContainer(&RequiredBone);
	Curve.InitFrom(RequiredBone);
}

/////////////////////////////////////////////////////
// FComponentSpacePoseContext

void FComponentSpacePoseContext::ResetToRefPose()
{
	checkSlow(AnimInstanceProxy && AnimInstanceProxy->GetRequiredBones().IsValid());
	const FBoneContainer& RequiredBone = AnimInstanceProxy->GetRequiredBones();
	Pose.InitPose(&RequiredBone);
	Curve.InitFrom(RequiredBone);
}

/////////////////////////////////////////////////////
// FAnimNode_Base

void FAnimNode_Base::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
}

void FAnimNode_Base::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
}

void FAnimNode_Base::Update_AnyThread(const FAnimationUpdateContext& Context)
{
}

void FAnimNode_Base::Evaluate_AnyThread(FPoseContext& Output)
{
}

void FAnimNode_Base::EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output)
{
}

bool FAnimNode_Base::IsLODEnabled(FAnimInstanceProxy* AnimInstanceProxy)
{
	const int32 NodeLODThreshold = GetLODThreshold();
	return ((NodeLODThreshold == INDEX_NONE) || (AnimInstanceProxy->GetLODLevel() <= NodeLODThreshold));
}

void FAnimNode_Base::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
}

void FAnimNode_Base::ResetDynamics(ETeleportType InTeleportType)
{
	// Call legacy implementation for backwards compatibility
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ResetDynamics();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FAnimNodeFunctionRef& FAnimNode_Base::GetInitialUpdateFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, InitialUpdateFunction);
}

const FAnimNodeFunctionRef& FAnimNode_Base::GetBecomeRelevantFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, BecomeRelevantFunction);
}

const FAnimNodeFunctionRef& FAnimNode_Base::GetUpdateFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, UpdateFunction);
}

/////////////////////////////////////////////////////
// FPoseLinkBase

void FPoseLinkBase::AttemptRelink(const FAnimationBaseContext& Context)
{
	// Do the linkage
	if ((LinkedNode == nullptr) && (LinkID != INDEX_NONE))
	{
		IAnimClassInterface* AnimBlueprintClass = Context.GetAnimClass();
		check(AnimBlueprintClass);

		// adding ensure. We had a crash on here
		const TArray<FStructProperty*>& AnimNodeProperties = AnimBlueprintClass->GetAnimNodeProperties();
		if ( ensure(AnimNodeProperties.IsValidIndex(LinkID)) )
		{
			FStructProperty* LinkedProperty = AnimNodeProperties[LinkID];
			void* LinkedNodePtr = LinkedProperty->ContainerPtrToValuePtr<void>(Context.AnimInstanceProxy->GetAnimInstanceObject());
			LinkedNode = (FAnimNode_Base*)LinkedNodePtr;
		}
	}
}

void FPoseLinkBase::Initialize(const FAnimationInitializeContext& InContext)
{
#if DO_CHECK
	checkf(!bProcessed, TEXT("Initialize already in progress, circular link for AnimInstance [%s] Blueprint [%s]"), \
		*InContext.AnimInstanceProxy->GetAnimInstanceName(), *GetFullNameSafe(IAnimClassInterface::GetActualAnimClass(InContext.AnimInstanceProxy->GetAnimClassInterface())));
	TGuardValue<bool> CircularGuard(bProcessed, true);
#endif

	AttemptRelink(InContext);

#if ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG
	InitializationCounter.SynchronizeWith(InContext.AnimInstanceProxy->GetInitializationCounter());

	// Initialization will require update to be called before an evaluate.
	UpdateCounter.Reset();
#endif

	// Do standard initialization
	if (LinkedNode != nullptr)
	{
		FAnimationInitializeContext LinkContext(InContext);
		LinkContext.SetNodeId(LinkID);
		TRACE_SCOPED_ANIM_NODE(LinkContext);
		UE_DONT_INLINE_CALL LinkedNode->Initialize_AnyThread(LinkContext);
	}
}

void FPoseLinkBase::SetLinkNode(struct FAnimNode_Base* NewLinkNode)
{
	// this is custom interface, only should be used by native handlers
	LinkedNode = NewLinkNode;
}

void FPoseLinkBase::SetDynamicLinkNode(struct FPoseLinkBase* InPoseLink)
{
	if(InPoseLink)
	{
		LinkedNode = InPoseLink->LinkedNode;
#if WITH_EDITORONLY_DATA
		SourceLinkID = InPoseLink->SourceLinkID;
#endif
		LinkID = InPoseLink->LinkID;
	}
	else
	{
		LinkedNode = nullptr;
#if WITH_EDITORONLY_DATA
		SourceLinkID = INDEX_NONE;
#endif
		LinkID = INDEX_NONE;
	}
}

FAnimNode_Base* FPoseLinkBase::GetLinkNode()
{
	return LinkedNode;
}

// Don't inline this function to keep the stack usage down
FORCENOINLINE const FExposedValueHandler& FAnimNode_Base::GetEvaluateGraphExposedInputs() const
{
	if(NodeData)
	{
		const int32 NodeIndex = NodeData->GetNodeIndex();
		return NodeData->GetAnimClassInterface().GetSubsystem<FAnimSubsystem_Base>().GetExposedValueHandlers()[NodeIndex];
	}
	else
	{
		// Inverting control (entering via the immutable data rather than the mutable data) would allow
		// us to remove this static local. Would also allow us to remove the vtable from FAnimNode_Base.
		static const FExposedValueHandler Default;	
		return Default;
	}
}

void FPoseLinkBase::CacheBones(const FAnimationCacheBonesContext& InContext) 
{
#if DO_CHECK
	checkf( !bProcessed, TEXT( "CacheBones already in progress, circular link for AnimInstance [%s] Blueprint [%s]" ), \
		*InContext.AnimInstanceProxy->GetAnimInstanceName(), *GetFullNameSafe(IAnimClassInterface::GetActualAnimClass(InContext.AnimInstanceProxy->GetAnimClassInterface())));
	TGuardValue<bool> CircularGuard(bProcessed, true);
#endif

#if ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG
	CachedBonesCounter.SynchronizeWith(InContext.AnimInstanceProxy->GetCachedBonesCounter());
#endif

	if (LinkedNode != nullptr)
	{
		UE_DONT_INLINE_CALL LinkedNode->CacheBones_AnyThread(InContext);
	}
}

void FPoseLinkBase::Update(const FAnimationUpdateContext& InContext)
{
#if ENABLE_VERBOSE_ANIM_PERF_TRACKING
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FPoseLinkBase_Update);
#endif // ENABLE_VERBOSE_ANIM_PERF_TRACKING

#if DO_CHECK
	checkf( !bProcessed, TEXT( "Update already in progress, circular link for AnimInstance [%s] Blueprint [%s]" ), \
		*InContext.AnimInstanceProxy->GetAnimInstanceName(), *GetFullNameSafe(IAnimClassInterface::GetActualAnimClass(InContext.AnimInstanceProxy->GetAnimClassInterface())));
	TGuardValue<bool> CircularGuard(bProcessed, true);
#endif

#if WITH_EDITOR
	if (GIsEditor)
	{
		if (LinkedNode == nullptr)
		{
			//@TODO: Should only do this when playing back
			AttemptRelink(InContext);
		}

		// Record the node line activation
		if (LinkedNode != nullptr)
		{
			if (InContext.AnimInstanceProxy->IsBeingDebugged())
			{
				InContext.AnimInstanceProxy->RecordNodeVisit(LinkID, SourceLinkID, InContext.GetFinalBlendWeight());
			}
		}
	}
#endif

#if ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG
	checkf(InitializationCounter.IsSynchronized_Counter(InContext.AnimInstanceProxy->GetInitializationCounter()), TEXT("Calling Update without initialization!"));
	UpdateCounter.SynchronizeWith(InContext.AnimInstanceProxy->GetUpdateCounter());
#endif

	if (LinkedNode != nullptr)
	{
		FAnimationUpdateContext LinkContext(InContext.WithNodeId(LinkID));
		TRACE_SCOPED_ANIM_NODE(LinkContext);
		if(LinkedNode->NodeData && LinkedNode->NodeData->HasNodeAnyFlags(EAnimNodeDataFlags::AllFunctions))
		{
			UE::Anim::FNodeFunctionCaller::InitialUpdate(LinkContext, *LinkedNode);
			UE::Anim::FNodeFunctionCaller::BecomeRelevant(LinkContext, *LinkedNode);
			UE::Anim::FNodeFunctionCaller::Update(LinkContext, *LinkedNode);
		}
		UE_DONT_INLINE_CALL LinkedNode->Update_AnyThread(LinkContext);
	}
}

void FPoseLinkBase::GatherDebugData(FNodeDebugData& InDebugData)
{
	if(LinkedNode != nullptr)
	{
		LinkedNode->GatherDebugData(InDebugData);
	}
}

/////////////////////////////////////////////////////
// FPoseLink

void FPoseLink::Evaluate(FPoseContext& Output)
{
#if DO_CHECK
	checkf( !bProcessed, TEXT( "Evaluate already in progress, circular link for AnimInstance [%s] Blueprint [%s]" ), \
		*Output.AnimInstanceProxy->GetAnimInstanceName(), *GetFullNameSafe(IAnimClassInterface::GetActualAnimClass(Output.AnimInstanceProxy->GetAnimClassInterface())));
	TGuardValue<bool> CircularGuard(bProcessed, true);
#endif

#if WITH_EDITOR
	if ((LinkedNode == nullptr) && GIsEditor)
	{
		//@TODO: Should only do this when playing back
		AttemptRelink(Output);
	}
#endif

#if ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG
	checkf(InitializationCounter.IsSynchronized_Counter(Output.AnimInstanceProxy->GetInitializationCounter()), TEXT("Calling Evaluate without initialization!"));
	checkf(UpdateCounter.IsSynchronized_Counter(Output.AnimInstanceProxy->GetUpdateCounter()), TEXT("Calling Evaluate without Update for this node!"));
	checkf(CachedBonesCounter.IsSynchronized_Counter(Output.AnimInstanceProxy->GetCachedBonesCounter()), TEXT("Calling Evaluate without CachedBones!"));
	EvaluationCounter.SynchronizeWith(Output.AnimInstanceProxy->GetEvaluationCounter());
#endif

	if (LinkedNode != nullptr)
	{
#if ENABLE_ANIMNODE_POSE_DEBUG
		CurrentPose.ResetToAdditiveIdentity();
#endif

		int32 SourceID = Output.GetCurrentNodeId();

		{
			Output.SetNodeId(LinkID);
			TRACE_SCOPED_ANIM_NODE(Output);
			UE_DONT_INLINE_CALL LinkedNode->Evaluate_AnyThread(Output);
			TRACE_ANIM_NODE_BLENDABLE_ATTRIBUTES(Output, SourceID, LinkID);
		}

#if WITH_EDITORONLY_DATA
		if (Output.AnimInstanceProxy->IsBeingDebugged())
		{
			if(Output.CustomAttributes.ContainsData())
			{
				Output.AnimInstanceProxy->RecordNodeAttribute(*Output.AnimInstanceProxy, SourceID, LinkID, UE::Anim::FAttributes::Attributes);

				TArray<FName, TInlineAllocator<8>> AttributeKeyNames;
				if (Output.CustomAttributes.GetAllKeyNames(AttributeKeyNames))
				{
					for (const FName& AttributeKeyName : AttributeKeyNames)
					{
						Output.AnimInstanceProxy->RecordNodeAttribute(*Output.AnimInstanceProxy, SourceID, LinkID, AttributeKeyName);
					}
				}
			}
			if(Output.Curve.Num() > 0)
			{
				Output.AnimInstanceProxy->RecordNodeAttribute(*Output.AnimInstanceProxy, SourceID, LinkID, UE::Anim::FAttributes::Curves);
			}
		}
#endif

#if ENABLE_ANIMNODE_POSE_DEBUG
		CurrentPose.CopyBonesFrom(Output.Pose);
#endif

#if WITH_EDITOR
		Output.AnimInstanceProxy->RegisterWatchedPose(Output, LinkID);
#endif
	}
	else
	{
		//@TODO: Warning here?
		Output.ResetToRefPose();
	}

	// Detect non valid output
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE::Anim::Private::ValidatePose(Output.Pose, Output.AnimInstanceProxy, LinkedNode);
#endif
}

/////////////////////////////////////////////////////
// FComponentSpacePoseLink

void FComponentSpacePoseLink::EvaluateComponentSpace(FComponentSpacePoseContext& Output)
{
#if DO_CHECK
	checkf( !bProcessed, TEXT( "EvaluateComponentSpace already in progress, circular link for AnimInstance [%s] Blueprint [%s]" ), \
		*Output.AnimInstanceProxy->GetAnimInstanceName(), *GetFullNameSafe(IAnimClassInterface::GetActualAnimClass(Output.AnimInstanceProxy->GetAnimClassInterface())));
	TGuardValue<bool> CircularGuard(bProcessed, true);
#endif

#if ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG
	checkf(InitializationCounter.IsSynchronized_Counter(Output.AnimInstanceProxy->GetInitializationCounter()), TEXT("Calling EvaluateComponentSpace without initialization!"));
	checkf(CachedBonesCounter.IsSynchronized_Counter(Output.AnimInstanceProxy->GetCachedBonesCounter()), TEXT("Calling EvaluateComponentSpace without CachedBones!"));
	checkf(UpdateCounter.IsSynchronized_Counter(Output.AnimInstanceProxy->GetUpdateCounter()), TEXT("Calling EvaluateComponentSpace without Update for this node!"));
	EvaluationCounter.SynchronizeWith(Output.AnimInstanceProxy->GetEvaluationCounter());
#endif

	if (LinkedNode != nullptr)
	{
		int32 SourceID = Output.GetCurrentNodeId();

		{
			Output.SetNodeId(LinkID);
			TRACE_SCOPED_ANIM_NODE(Output);
			UE_DONT_INLINE_CALL LinkedNode->EvaluateComponentSpace_AnyThread(Output);
			TRACE_ANIM_NODE_BLENDABLE_ATTRIBUTES(Output, SourceID, LinkID);
		}

#if WITH_EDITORONLY_DATA
		if (Output.AnimInstanceProxy->IsBeingDebugged())
		{
			if(Output.CustomAttributes.ContainsData())
			{
				Output.AnimInstanceProxy->RecordNodeAttribute(*Output.AnimInstanceProxy, SourceID, LinkID, UE::Anim::FAttributes::Attributes);

				TArray<FName, TInlineAllocator<8>> AttributeKeyNames;
				if (Output.CustomAttributes.GetAllKeyNames(AttributeKeyNames))
				{
					for (const FName& AttributeKeyName : AttributeKeyNames)
					{
						Output.AnimInstanceProxy->RecordNodeAttribute(*Output.AnimInstanceProxy, SourceID, LinkID, AttributeKeyName);
					}
				}
			}
			if(Output.Curve.Num() > 0)
			{
				Output.AnimInstanceProxy->RecordNodeAttribute(*Output.AnimInstanceProxy, SourceID, LinkID, UE::Anim::FAttributes::Curves);
			}
		}
#endif

#if WITH_EDITOR
		Output.AnimInstanceProxy->RegisterWatchedPose(Output, LinkID);
#endif
	}
	else
	{
		//@TODO: Warning here?
		Output.ResetToRefPose();
	}

	// Detect non valid output
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE::Anim::Private::ValidatePose(Output.Pose.GetPose(), Output.AnimInstanceProxy, LinkedNode);
#endif
}

/////////////////////////////////////////////////////
// FComponentSpacePoseContext

bool FComponentSpacePoseContext::ContainsNaN() const
{
	return Pose.GetPose().ContainsNaN();
}

bool FComponentSpacePoseContext::IsNormalized() const
{
	return Pose.GetPose().IsNormalized();
}

/////////////////////////////////////////////////////
// FNodeDebugData

void FNodeDebugData::AddDebugItem(FString DebugData, bool bPoseSource)
{
	check(NodeChain.Num() == 0 || NodeChain.Last().ChildNodeChain.Num() == 0); //Cannot add to this chain once we have branched

	NodeChain.Add( DebugItem(DebugData, bPoseSource) );
	NodeChain.Last().ChildNodeChain.Reserve(ANIM_NODE_DEBUG_MAX_CHILDREN);
}

FNodeDebugData& FNodeDebugData::BranchFlow(float BranchWeight, FString InNodeDescription)
{
	NodeChain.Last().ChildNodeChain.Add(FNodeDebugData(AnimInstance, BranchWeight*AbsoluteWeight, InNodeDescription, RootNodePtr));
	NodeChain.Last().ChildNodeChain.Last().NodeChain.Reserve(ANIM_NODE_DEBUG_MAX_CHAIN);
	return NodeChain.Last().ChildNodeChain.Last();
}

FNodeDebugData* FNodeDebugData::GetCachePoseDebugData(float GlobalWeight)
{
	check(RootNodePtr);

	RootNodePtr->SaveCachePoseNodes.Add( FNodeDebugData(AnimInstance, GlobalWeight, FString(), RootNodePtr) );
	RootNodePtr->SaveCachePoseNodes.Last().NodeChain.Reserve(ANIM_NODE_DEBUG_MAX_CHAIN);
	return &RootNodePtr->SaveCachePoseNodes.Last();
}

void FNodeDebugData::GetFlattenedDebugData(TArray<FFlattenedDebugData>& FlattenedDebugData, int32 Indent, int32& ChainID)
{
	int32 CurrChainID = ChainID;
	for(DebugItem& Item : NodeChain)
	{
		FlattenedDebugData.Add( FFlattenedDebugData(Item.DebugData, AbsoluteWeight, Indent, CurrChainID, Item.bPoseSource) );
		bool bMultiBranch = Item.ChildNodeChain.Num() > 1;
		int32 ChildIndent = bMultiBranch ? Indent + 1 : Indent;
		for(FNodeDebugData& Child : Item.ChildNodeChain)
		{
			if(bMultiBranch)
			{
				// If we only have one branch we treat it as the same really
				// as we may have only changed active status
				++ChainID;
			}
			Child.GetFlattenedDebugData(FlattenedDebugData, ChildIndent, ChainID);
		}
	}

	// Do CachePose nodes only from the root.
	if (RootNodePtr == this)
	{
		for (FNodeDebugData& CachePoseData : SaveCachePoseNodes)
		{
			++ChainID;
			CachePoseData.GetFlattenedDebugData(FlattenedDebugData, 0, ChainID);
		}
	}
}

