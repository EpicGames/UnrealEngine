// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNodeData.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimNodeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNodeData)

namespace UE { namespace Anim {

FNodeDataId::FNodeDataId(FName InPropertyName, const FAnimNode_Base* InNode, const UScriptStruct* InNodeStruct)
{
	check(InPropertyName != NAME_None);
	check(InNode);
	check(InNodeStruct);

	Id = InPropertyName;

	if(InNode->NodeData)
	{
#if WITH_EDITORONLY_DATA
		check(InNode->NodeData->GetAnimClassInterface().IsDataLayoutValid());
#endif
		
		Index = InNode->NodeData->GetAnimClassInterface().GetAnimNodePropertyIndex(InNodeStruct, InPropertyName);
		check(InNode->NodeData->Entries.IsValidIndex(Index));
		check(InNode->NodeData->Entries[Index] != ANIM_NODE_DATA_INVALID_ENTRY);	
	}

#if WITH_EDITORONLY_DATA
	check(InNodeStruct);
	Struct = InNodeStruct;

	Property = InNodeStruct->FindPropertyByName(InPropertyName);
	check(Property);
#endif
}

} }

const void* FAnimNodeData::GetData(UE::Anim::FNodeDataId InId, const FAnimNode_Base* InNode, const UObject* InCurrentObject) const
{
	check(Entries.IsValidIndex(InId.Index));
	
	const uint32 Entry = Entries[InId.Index];
	check(Entry != ANIM_NODE_DATA_INVALID_ENTRY);
	
	uint32 EntryIndex = (Entry & ANIM_NODE_DATA_INSTANCE_DATA_MASK);
	if((Entry & ANIM_NODE_DATA_INSTANCE_DATA_FLAG) != 0)
	{
		// Use the supplied object or find the object ptr by walking the property chain from this node
		const UObject* CurrentObject = InCurrentObject ? InCurrentObject : IAnimClassInterface::GetObjectPtrFromAnimNode(&(*AnimClassInterface), InNode);

		// Check the current context is expected
		check(CurrentObject && CurrentObject->GetClass()->IsChildOf(IAnimClassInterface::GetActualAnimClass(&(*AnimClassInterface))));

		return AnimClassInterface->GetMutableNodeValueRaw(EntryIndex, CurrentObject);
	}
	else
	{
		return AnimClassInterface->GetConstantNodeValueRaw(EntryIndex);
	}
}

#if WITH_EDITORONLY_DATA
void* FAnimNodeData::GetMutableData(UE::Anim::FNodeDataId InId, FAnimNode_Base* InNode, UObject* InCurrentObject) const
{
	return const_cast<void*>(GetData(InId, const_cast<FAnimNode_Base*>(InNode), const_cast<UObject*>(InCurrentObject))); 
}
#endif

void* FAnimNodeData::GetInstanceData(UE::Anim::FNodeDataId InId, FAnimNode_Base* InNode, UObject* InCurrentObject) const
{
	check(Entries.IsValidIndex(InId.Index));
	
	const uint32 Entry = Entries[InId.Index];
	check(Entry != ANIM_NODE_DATA_INVALID_ENTRY);
	
	uint32 EntryIndex = (Entry & ANIM_NODE_DATA_INSTANCE_DATA_MASK);
	if((Entry & ANIM_NODE_DATA_INSTANCE_DATA_FLAG) != 0)
	{
		// Use the supplied object or find the object ptr by walking the property chain from this node
		const UObject* CurrentObject = InCurrentObject ? InCurrentObject : IAnimClassInterface::GetObjectPtrFromAnimNode(&(*AnimClassInterface), InNode);

		// Check the current context is expected
		check(CurrentObject && CurrentObject->GetClass()->IsChildOf(IAnimClassInterface::GetActualAnimClass(&(*AnimClassInterface))));

		return const_cast<void*>(AnimClassInterface->GetMutableNodeValueRaw(EntryIndex, CurrentObject));
	}

	return nullptr;
}

FAnimNodeStructData::FAnimNodeStructData(const UScriptStruct* InNodeType)
{
	// Iterated properties are not in the same order they are laid out in memory, so we extract and sort them here.
	TArray<FProperty*> Properties;
	for(TFieldIterator<FProperty> It(InNodeType); It; ++It)
	{
		Properties.Add(*It);
	}

	Algo::Sort(Properties, [](const FProperty* InProperty0, const FProperty* InProperty1)
    {
        return InProperty0->GetOffset_ForInternal() < InProperty1->GetOffset_ForInternal();
    });

	int32 PropertyIndex = 0;
	for(; PropertyIndex < Properties.Num(); ++PropertyIndex)
	{
		NameToIndexMap.Add(Properties[PropertyIndex]->GetFName(), PropertyIndex);
	}

	NumProperties = PropertyIndex;
}

int32 FAnimNodeStructData::GetPropertyIndex(FName InPropertyName) const
{
	const int32* IndexPtr = NameToIndexMap.Find(InPropertyName);
	return IndexPtr ? *IndexPtr : INDEX_NONE;
}

int32 FAnimNodeStructData::GetNumProperties() const
{
	return NumProperties;
}

#if WITH_EDITORONLY_DATA
bool FAnimNodeStructData::DoesLayoutMatch(const FAnimNodeStructData& InOther) const
{
	return NameToIndexMap.OrderIndependentCompareEqual(InOther.NameToIndexMap);
}
#endif
