// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNodeReference.h"
#include "Animation/AnimSubsystem_NodeRelevancy.h"
#include "Animation/AnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNodeFunctionRef)

void FAnimNodeFunctionRef::SetFromFunction(UFunction* InFunction)
{
	if(InFunction)
	{
		FunctionName = InFunction->GetFName();
		UClass* OwnerClass = InFunction->GetOwnerClass();
		// Only need the class name if the function is static, as the function will be called on the passed-in UObject otherwise.
		if(OwnerClass && InFunction->HasAnyFunctionFlags(FUNC_Static))
		{
			ensureAlwaysMsgf(OwnerClass->IsChildOf(UAnimInstance::StaticClass()) || OwnerClass->IsChildOf(UBlueprintFunctionLibrary::StaticClass()), TEXT("Function class must derive from either UAnimInstance or UBlueprintFunctionLibrary"));
			ClassName = *OwnerClass->GetPathName();
		}
		else
		{
			ClassName = NAME_None;
		}
	}
	else
	{
		FunctionName = NAME_None;
		ClassName = NAME_None;
	}
}

void FAnimNodeFunctionRef::Initialize(const UClass* InClass)
{
	const UClass* ClassToUse = InClass;
	
	if(ClassName != NAME_None)
	{
		if(UClass* FoundClass = FindObject<UClass>(nullptr, *ClassName.ToString()))
		{
			Class = ClassToUse = FoundClass;
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Could not find function class %ls", *ClassName.ToString());
		}
	}
	
	if(FunctionName != NAME_None)
	{
		Function = ClassToUse->FindFunctionByName(FunctionName);
	}
}

void FAnimNodeFunctionRef::Call(UObject* InObject, void* InParameters) const
{
	if(IsValid())
	{
		if(Class)
		{
			Class->GetDefaultObject()->ProcessEvent(Function, InParameters);
		}
		else
		{
			InObject->ProcessEvent(Function, InParameters);
		}
	}
}

namespace UE
{
namespace Anim
{

template<typename WrapperType, typename ContextType>
static void CallFunctionHelper(const FAnimNodeFunctionRef& InFunction, const ContextType& InContext, FAnimNode_Base& InNode)
{
	struct FAnimNodeFunctionParams
	{
		WrapperType ExecutionContext;
		FAnimNodeReference NodeReference;
	};

	if(InFunction.IsValid())
	{
		if(ensureMsgf(InFunction.GetFunction()->ParmsSize == sizeof(FAnimNodeFunctionParams), TEXT("Function parameters size of %s does not match expected."), *InFunction.GetFunction()->GetName()))
		{
			UAnimInstance* AnimInstance = CastChecked<UAnimInstance>(InContext.GetAnimInstanceObject());
			TSharedRef<FAnimExecutionContext::FData> ContextData = MakeShared<FAnimExecutionContext::FData>(InContext);
			FAnimNodeFunctionParams Params = { WrapperType(ContextData), FAnimNodeReference(AnimInstance, InNode) };

			InFunction.Call(AnimInstance, &Params);
		}
	}
}

void FNodeFunctionCaller::InitialUpdate(const FAnimationUpdateContext& InContext, FAnimNode_Base& InNode)
{
	if(InNode.NodeData != nullptr && InNode.NodeData->HasNodeAnyFlags(EAnimNodeDataFlags::HasInitialUpdateFunction))
	{
		const FAnimNodeFunctionRef& Function = InNode.GetInitialUpdateFunction();
		if(Function.IsValid())
		{
			FAnimSubsystemInstance_NodeRelevancy& RelevancySubsystem = CastChecked<UAnimInstance>(InContext.GetAnimInstanceObject())->GetSubsystem<FAnimSubsystemInstance_NodeRelevancy>();
			const EAnimNodeInitializationStatus Status = RelevancySubsystem.UpdateNodeInitializationStatus(InContext, InNode);
			if(Status == EAnimNodeInitializationStatus::InitialUpdate)
			{
				CallFunctionHelper<FAnimUpdateContext>(Function, InContext, InNode);
			}
		}
	}
}

void FNodeFunctionCaller::BecomeRelevant(const FAnimationUpdateContext& InContext, FAnimNode_Base& InNode)
{
	if(InNode.NodeData != nullptr && InNode.NodeData->HasNodeAnyFlags(EAnimNodeDataFlags::HasBecomeRelevantFunction))
	{
		const FAnimNodeFunctionRef& Function = InNode.GetBecomeRelevantFunction();
		if(Function.IsValid())
		{
			FAnimSubsystemInstance_NodeRelevancy& RelevancySubsystem = CastChecked<UAnimInstance>(InContext.GetAnimInstanceObject())->GetSubsystem<FAnimSubsystemInstance_NodeRelevancy>();
			FAnimNodeRelevancyStatus Status = RelevancySubsystem.UpdateNodeRelevancy(InContext, InNode);
			if(Status.HasJustBecomeRelevant())
			{
				CallFunctionHelper<FAnimUpdateContext>(Function, InContext, InNode);
			}
		}
	}
}

void FNodeFunctionCaller::Update(const FAnimationUpdateContext& InContext, FAnimNode_Base& InNode)
{
	if(InNode.NodeData != nullptr && InNode.NodeData->HasNodeAnyFlags(EAnimNodeDataFlags::HasUpdateFunction))
	{
		CallFunctionHelper<FAnimUpdateContext>(InNode.GetUpdateFunction(), InContext, InNode);
	}
}

void FNodeFunctionCaller::CallFunction(const FAnimNodeFunctionRef& InFunction, const FAnimationBaseContext& InContext, FAnimNode_Base& InNode)
{
	CallFunctionHelper<FAnimExecutionContext>(InFunction, InContext, InNode);
}

}}
