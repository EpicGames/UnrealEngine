// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNodeMessages.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimInstanceProxy.h"

namespace UE { namespace Anim {
const FName IGraphMessage::BaseTypeName("IGraphMessage");
	
const FName IAnimNotifyEventContextDataInterface::BaseTypeName("IAnimNotifyEventContextDataInterface");

FScopedGraphTag::FScopedGraphTag(const FAnimationBaseContext& InContext, FName InTag)
	: SharedContext(InContext.SharedContext)
	, Tag(InTag)
{
	check(SharedContext);

	int32 NodeId = InContext.GetCurrentNodeId();
	UScriptStruct* Struct = nullptr;
	FAnimNode_Base* AnimNode = nullptr;
	if(NodeId != INDEX_NONE)
	{
		if(IAnimClassInterface* Interface = InContext.AnimInstanceProxy->GetAnimClassInterface())
		{
			FStructProperty* NodeProperty = Interface->GetAnimNodeProperties()[NodeId];
			Struct = NodeProperty->Struct;
			AnimNode = NodeProperty->ContainerPtrToValuePtr<FAnimNode_Base>(InContext.AnimInstanceProxy->GetAnimInstanceObject());
		}
	}

	SharedContext->MessageStack.PushTag(AnimNode, Struct, Tag);
}

FScopedGraphTag::~FScopedGraphTag()
{
	check(SharedContext);
	SharedContext->MessageStack.PopTag(Tag);
}

FScopedGraphMessage::FScopedGraphMessage(const FAnimationBaseContext& InContext)
	: SharedContext(InContext.SharedContext)
{
}

void FScopedGraphMessage::PushMessage(const FAnimationBaseContext& InContext, TSharedRef<IGraphMessage> InMessage, FName InMessageType)
{
	check(SharedContext);

	MessageType = InMessageType;

	int32 NodeId = InContext.GetCurrentNodeId();
	UScriptStruct* Struct = nullptr;
	FAnimNode_Base* AnimNode = nullptr;
	if(NodeId != INDEX_NONE)
	{
		if(IAnimClassInterface* Interface = InContext.AnimInstanceProxy->GetAnimClassInterface())
		{
			FStructProperty* NodeProperty = Interface->GetAnimNodeProperties()[NodeId];
			Struct = NodeProperty->Struct;
			AnimNode = NodeProperty->ContainerPtrToValuePtr<FAnimNode_Base>(InContext.AnimInstanceProxy->GetAnimInstanceObject());
		}
	}

	SharedContext->MessageStack.PushMessage(MessageType, AnimNode, Struct, InMessage);
}

void FScopedGraphMessage::PopMessage()
{
	check(SharedContext);
	SharedContext->MessageStack.PopMessage(MessageType);
}

void FMessageStack::PushMessage(FName InMessageType, FAnimNode_Base* InNode, const UScriptStruct* InStruct, TSharedRef<IGraphMessage> InMessage)
{
	MessageStacks.FindOrAdd(InMessageType).Push( { InMessage, { InNode, InStruct } } );
}

void FMessageStack::PopMessage(FName InMessageType)
{
	MessageStacks.FindChecked(InMessageType).Pop();
}

void FMessageStack::ForEachMessage(FName InMessageType, TFunctionRef<EEnumerate(IGraphMessage&)> InFunction) const
{
	if(const FMessageStackEntry* StackEntryPtr = MessageStacks.Find(InMessageType))
	{
		const FMessageStackEntry& StackEntry = *StackEntryPtr;
		const int32 StackSize = StackEntry.Num();
		for(int32 MessageIndex = StackSize - 1; MessageIndex >= 0; --MessageIndex)
		{
			const FMessageEntry& MessageEntry = StackEntry[MessageIndex];
			if(MessageEntry.Message->GetTypeName() == InMessageType)
			{
				if(InFunction(*MessageEntry.Message) == EEnumerate::Stop)
				{
					return;
				}
			}
		}
	}
}

void FMessageStack::TopMessage(FName InMessageType, TFunctionRef<void(IGraphMessage&)> InFunction) const
{
	if(const FMessageStackEntry* StackEntryPtr = MessageStacks.Find(InMessageType))
	{
		const FMessageStackEntry& StackEntry = *StackEntryPtr;
		if(StackEntry.Num())
		{
			const FMessageEntry& MessageEntry = StackEntry.Top();
			if(MessageEntry.Message->GetTypeName() == InMessageType)
			{
				InFunction(*MessageEntry.Message);
			}
		}
	}
}

void FMessageStack::TopMessageSharedPtr(FName InMessageType, TFunctionRef<void(TSharedPtr<IGraphMessage>&)> InFunction)
{
	if (FMessageStackEntry* StackEntryPtr = MessageStacks.Find(InMessageType))
	{
		FMessageStackEntry& StackEntry = *StackEntryPtr;
		if (StackEntry.Num())
		{
			FMessageEntry& MessageEntry = StackEntry.Top();
			if (MessageEntry.Message->GetTypeName() == InMessageType)
			{
				InFunction(MessageEntry.Message);
			}
		}
	}
}

void FMessageStack::BottomMessage(FName InMessageType, TFunctionRef<void(IGraphMessage&)> InFunction) const
{
	if(const FMessageStackEntry* StackEntryPtr = MessageStacks.Find(InMessageType))
	{
		const FMessageStackEntry& StackEntry = *StackEntryPtr;
		if(StackEntry.Num())
		{
			const FMessageEntry& MessageEntry = StackEntry[0];
			if(MessageEntry.Message->GetTypeName() == InMessageType)
			{
				InFunction(*MessageEntry.Message);
			}
		}
	}
}

bool FMessageStack::HasMessage(FName InMessageType) const
{
	if (const FMessageStackEntry* StackEntryPtr = MessageStacks.Find(InMessageType))
	{
		return !StackEntryPtr->IsEmpty();
	}

	return false;
}

void FMessageStack::PushTag(FAnimNode_Base* InNode, const UScriptStruct* InStruct, FName InTag)
{
	TagStacks.FindOrAdd(InTag).Push( { InNode, InStruct } );
}

void FMessageStack::PopTag(FName InTag)
{
	TagStacks.FindChecked(InTag).Pop();
}

void FMessageStack::ForEachTag(FName InTagId, TFunctionRef<EEnumerate(FNodeInfo)> InFunction) const
{
	if(const FTagStackEntry* TagStackPtr = TagStacks.Find(InTagId))
	{
		const FTagStackEntry& StackEntry = *TagStackPtr;
		const int32 StackSize = StackEntry.Num();
		for(int32 TagIndex = StackSize - 1; TagIndex >= 0; --TagIndex)
		{
			const FNodeInfo& NodeInfo = StackEntry[TagIndex];
			if(InFunction(NodeInfo) == EEnumerate::Stop)
			{
				return;
			}
		}
	}
}

void FMessageStack::CopyForCachedUpdate(const FMessageStack& InStack)
{
	MessageStacks.Reset();
	MessageStacks.Reserve(InStack.MessageStacks.Num());
	for (const TPair<FName, FMessageStackEntry>& MessageStackPair : InStack.MessageStacks)
	{
		if (MessageStackPair.Value.Num() > 0)
		{
			FMessageStackEntry& Stack = MessageStacks.Add(MessageStackPair.Key);
			Stack.Push(MessageStackPair.Value.Top());
		}
	}

	TagStacks.Reset();
	TagStacks.Reserve(InStack.TagStacks.Num());
	for (const TPair<FName, FTagStackEntry>& TagStackPair : InStack.TagStacks)
	{
		if (TagStackPair.Value.Num() > 0)
		{
			FTagStackEntry& Stack = TagStacks.Add(TagStackPair.Key);
			Stack.Push(TagStackPair.Value.Top());
		}
	}
}

void FMessageStack::MakeEventContextData(TArray<TUniquePtr<const IAnimNotifyEventContextDataInterface>>& ContextData) const
{
	ContextData.Empty();
	for (const TPair<FName, FMessageStackEntry>& MessageStackPair : MessageStacks)
	{
		const FMessageStackEntry& StackEntry = MessageStackPair.Value;
		if(StackEntry.Num())
		{
			const FMessageEntry& MessageEntry = StackEntry.Top();
			if(TUniquePtr<const IAnimNotifyEventContextDataInterface> NotifyData = MessageEntry.Message->MakeUniqueEventContextData())
			{
				ContextData.Add(MoveTemp(NotifyData));
			}
		}
	}
}

}}	// namespace UE::Anim