// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNodeReference.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimClassInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNodeReference)

FAnimNodeReference::FAnimNodeReference(UAnimInstance* InAnimInstance, FAnimNode_Base& InNode)
	: AnimNode(&InNode)
{
	IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(InAnimInstance->GetClass());
	const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
	AnimNodeStruct = AnimNodeProperties[InNode.GetNodeIndex()]->Struct;
}

FAnimNodeReference::FAnimNodeReference(UAnimInstance* InAnimInstance, int32 InIndex)
{
	IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(InAnimInstance->GetClass());
	const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();
	if(AnimNodeProperties.IsValidIndex(InIndex))
	{
		AnimNodeStruct = AnimNodeProperties[InIndex]->Struct;
		AnimNode = AnimNodeProperties[InIndex]->ContainerPtrToValuePtr<FAnimNode_Base>(InAnimInstance);
	}
}
