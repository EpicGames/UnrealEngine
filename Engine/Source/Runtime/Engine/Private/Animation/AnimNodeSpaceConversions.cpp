// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNodeSpaceConversions.h"
#include "Animation/AnimStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNodeSpaceConversions)

/////////////////////////////////////////////////////
// FAnimNode_ConvertComponentToLocalSpace

FAnimNode_ConvertComponentToLocalSpace::FAnimNode_ConvertComponentToLocalSpace()
{
}

void FAnimNode_ConvertComponentToLocalSpace::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	ComponentPose.Initialize(Context);
}

void FAnimNode_ConvertComponentToLocalSpace::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) 
{
	ComponentPose.CacheBones(Context);
}

void FAnimNode_ConvertComponentToLocalSpace::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	ComponentPose.Update(Context);
}

void FAnimNode_ConvertComponentToLocalSpace::Evaluate_AnyThread(FPoseContext & Output)
{
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ConvertComponentToLocalSpace, !IsInGameThread());

	// Evaluate the child and convert
	FComponentSpacePoseContext InputCSPose(Output.AnimInstanceProxy);

	// We need to preserve the node ID chain as we use the proxy-based constructor above
	InputCSPose.SetNodeIds(Output);

	ComponentPose.EvaluateComponentSpace(InputCSPose);

	checkSlow( InputCSPose.Pose.GetPose().IsValid() );
	FCSPose<FCompactPose>::ConvertComponentPosesToLocalPoses(MoveTemp(InputCSPose.Pose), Output.Pose);
	Output.Curve = MoveTemp(InputCSPose.Curve);
	Output.CustomAttributes = MoveTemp(InputCSPose.CustomAttributes);
}

void FAnimNode_ConvertComponentToLocalSpace::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugData.AddDebugItem(DebugLine);
	ComponentPose.GatherDebugData(DebugData);
}

/////////////////////////////////////////////////////
// FAnimNode_ConvertLocalToComponentSpace

FAnimNode_ConvertLocalToComponentSpace::FAnimNode_ConvertLocalToComponentSpace()
{
}

void FAnimNode_ConvertLocalToComponentSpace::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	LocalPose.Initialize(Context);
}

void FAnimNode_ConvertLocalToComponentSpace::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) 
{
	LocalPose.CacheBones(Context);
}

void FAnimNode_ConvertLocalToComponentSpace::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	LocalPose.Update(Context);
}

void FAnimNode_ConvertLocalToComponentSpace::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugData.AddDebugItem(DebugLine);
	LocalPose.GatherDebugData(DebugData);
}

void FAnimNode_ConvertLocalToComponentSpace::EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext & OutputCSPose)
{
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ConvertLocalToComponentSpace, !IsInGameThread());

	// Evaluate the child and convert
	FPoseContext InputPose(OutputCSPose.AnimInstanceProxy);

	// We need to preserve the node ID chain as we use the proxy-based constructor above
	InputPose.SetNodeIds(OutputCSPose);

	LocalPose.Evaluate(InputPose);

	OutputCSPose.Pose.InitPose(MoveTemp(InputPose.Pose));
	OutputCSPose.Curve = MoveTemp(InputPose.Curve);
	OutputCSPose.CustomAttributes = MoveTemp(InputPose.CustomAttributes);
}

