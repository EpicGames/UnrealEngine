// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_ApplyMeshSpaceAdditive.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AnimStats.h"
#include "Animation/ExposedValueHandler.h"
#include "AnimationRuntime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ApplyMeshSpaceAdditive)

/////////////////////////////////////////////////////
// FAnimNode_ApplyMeshSpaceAdditive

void FAnimNode_ApplyMeshSpaceAdditive::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	Base.Initialize(Context);
	Additive.Initialize(Context);

	AlphaBoolBlend.Reinitialize();
	AlphaScaleBiasClamp.Reinitialize();
}

void FAnimNode_ApplyMeshSpaceAdditive::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Base.CacheBones(Context);
	Additive.CacheBones(Context);
}

void FAnimNode_ApplyMeshSpaceAdditive::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAnimNode_ApplyMeshSpaceAdditive_Update);

	Base.Update(Context);

	if (!IsLODEnabled(Context.AnimInstanceProxy))
	{
		// Avoid doing work if we're not even going to be used.
		return;
	}

	GetEvaluateGraphExposedInputs().Execute(Context);

	ActualAlpha = 0.f;
	switch (AlphaInputType)
	{
	case EAnimAlphaInputType::Float:
		ActualAlpha = AlphaScaleBiasClamp.ApplyTo(AlphaScaleBias.ApplyTo(Alpha), Context.GetDeltaTime());
		break;
	case EAnimAlphaInputType::Bool:
		ActualAlpha = AlphaBoolBlend.ApplyTo(bAlphaBoolEnabled, Context.GetDeltaTime());
		break;
	case EAnimAlphaInputType::Curve:
		if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.AnimInstanceProxy->GetAnimInstanceObject()))
		{
			ActualAlpha = AlphaScaleBiasClamp.ApplyTo(AnimInstance->GetCurveValue(AlphaCurveName), Context.GetDeltaTime());
		}
		break;
	}

	ActualAlpha = FMath::Clamp(ActualAlpha, 0.0f, 1.0f);

	if (FAnimWeight::IsRelevant(ActualAlpha))
	{
		Additive.Update(Context.FractionalWeight(ActualAlpha));
	}
	
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Alpha"), ActualAlpha);
}

void FAnimNode_ApplyMeshSpaceAdditive::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ApplyMeshSpaceAdditive, !IsInGameThread());

	//@TODO: Could evaluate Base into Output and save a copy
	if (FAnimWeight::IsRelevant(ActualAlpha))
	{
		const bool bExpectsAdditivePose=true;
		FPoseContext AdditiveEvalContext(Output, bExpectsAdditivePose);

		Base.Evaluate(Output);
		Additive.Evaluate(AdditiveEvalContext);

		FAnimationPoseData BaseAnimationPoseData(Output);
		const FAnimationPoseData AdditiveAnimationPoseData(AdditiveEvalContext);

		
		if (bRootSpaceAdditive)
		{
			FAnimationRuntime::AccumulateRootSpaceRotationAdditiveToLocalPose(BaseAnimationPoseData, AdditiveAnimationPoseData, ActualAlpha);
		}
		else
		{
			FAnimationRuntime::AccumulateAdditivePose(BaseAnimationPoseData, AdditiveAnimationPoseData, ActualAlpha, AAT_RotationOffsetMeshSpace);
		}

		Output.Pose.NormalizeRotations();
	}
	else
	{
		Base.Evaluate(Output);
	}
}

FAnimNode_ApplyMeshSpaceAdditive::FAnimNode_ApplyMeshSpaceAdditive()
	: bRootSpaceAdditive(false)
	, AlphaInputType(EAnimAlphaInputType::Float)
	, Alpha(1.0f)
	, bAlphaBoolEnabled(true)
	, AlphaCurveName(NAME_None)
	, LODThreshold(INDEX_NONE)
	, ActualAlpha(0.f)
{
}

void FAnimNode_ApplyMeshSpaceAdditive::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Alpha: %.1f%%)"), ActualAlpha*100.f);

	DebugData.AddDebugItem(DebugLine);
	Base.GatherDebugData(DebugData.BranchFlow(1.f));
	Additive.GatherDebugData(DebugData.BranchFlow(ActualAlpha));
}

