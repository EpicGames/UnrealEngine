// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_AssetPlayerBase.h"

#include "Animation/AnimationInstanceScope.h"
#include "Animation/AnimSyncScope.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimInertializationSyncScope.h"
#include "Animation/AnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_AssetPlayerBase)

void FAnimNode_AssetPlayerBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_Base::Initialize_AnyThread(Context);

	MarkerTickRecord.Reset();
	bHasBeenFullWeight = false;
	
	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.GetAnimInstanceObject()))
	{
		AssetPlayerInstanceID = HashCombine(GetTypeHash(AnimInstance->AnimInstanceID), GetTypeHash(Context.GetCurrentNodeId()));
		bHasValidAssetPlayerInstanceID = true;
	}
}

void FAnimNode_AssetPlayerBase::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	// Cache the current weight and update the node
	BlendWeight = Context.GetFinalBlendWeight();
	bHasBeenFullWeight = bHasBeenFullWeight || (BlendWeight >= (1.0f - ZERO_ANIMWEIGHT_THRESH));

	UpdateAssetPlayer(Context);
}

void FAnimNode_AssetPlayerBase::CreateTickRecordForNode(const FAnimationUpdateContext& Context, UAnimSequenceBase* Sequence, bool bLooping, float PlayRate, bool bIsEvaluator)
{
	// Create a tick record and push into the closest scope
	const float FinalBlendWeight = Context.GetFinalBlendWeight();

	UE::Anim::FAnimSyncGroupScope& SyncScope = Context.GetMessageChecked<UE::Anim::FAnimSyncGroupScope>();

	// Active asset player's tick record
	FAnimTickRecord TickRecord(Sequence, bLooping, PlayRate, bIsEvaluator, FinalBlendWeight, /*inout*/ InternalTimeAccumulator, MarkerTickRecord);
	TickRecord.GatherContextData(Context);
	TickRecord.RootMotionWeightModifier = Context.GetRootMotionWeightModifier();
	TickRecord.DeltaTimeRecord = &DeltaTimeRecord;
	TickRecord.bRequestedInertialization = Context.GetMessage<UE::Anim::FAnimInertializationSyncScope>() != nullptr;
#if ANIM_TRACE_ENABLED
	TickRecord.SourceProxy = Context.AnimInstanceProxy;
#endif
	
	if (bHasValidAssetPlayerInstanceID)
	{
		TickRecord.MakeContextData<UE::Anim::FAnimNotifyAssetPlayerInstanceContext>(AssetPlayerInstanceID);
	}
	
	// Add asset player to synchronizer
	SyncScope.AddTickRecord(TickRecord, GetSyncParams(TickRecord.bRequestedInertialization), UE::Anim::FAnimSyncDebugInfo(Context));
}

float FAnimNode_AssetPlayerBase::GetAccumulatedTime() const
{
	return InternalTimeAccumulator;
}

void FAnimNode_AssetPlayerBase::SetAccumulatedTime(float NewTime)
{
	InternalTimeAccumulator = NewTime;
	MarkerTickRecord.Reset();
}

float FAnimNode_AssetPlayerBase::GetCachedBlendWeight() const
{
	return BlendWeight;
}

void FAnimNode_AssetPlayerBase::ClearCachedBlendWeight()
{
	BlendWeight = 0.0f;
}

float FAnimNode_AssetPlayerBase::GetCurrentAssetTimePlayRateAdjusted() const
{
	return GetCurrentAssetTime();
}

const FDeltaTimeRecord* FAnimNode_AssetPlayerBase::GetDeltaTimeRecord() const
{
	return &DeltaTimeRecord;
}

UE::Anim::FAnimSyncParams FAnimNode_AssetPlayerBase::GetSyncParams(bool bRequestedInertialization) const
{
	const EAnimGroupRole::Type SyncGroupRole = GetGroupRole();
	const FName SyncGroupName = GetGroupName();
	FName GroupNameToUse = SyncGroupName;
	EAnimSyncMethod MethodToUse = GetGroupMethod();
	bool bOverridePositionWhenJoiningSyncGroupAsLeader = GetOverridePositionWhenJoiningSyncGroupAsLeader();
	
	// Skip sync based on roles.
	{
		// Only allow transition leader/follower part of a sync group once after inertialization request. (Inertilization)
		if (bRequestedInertialization)
		{
			if (SyncGroupRole == EAnimGroupRole::TransitionLeader || SyncGroupRole == EAnimGroupRole::TransitionFollower)
			{
				GroupNameToUse = NAME_None;
			}
		}
		// Only allow transition leader/follower part of a sync group once it has full weight (Standard blend).
		else if ((SyncGroupRole == EAnimGroupRole::TransitionLeader || SyncGroupRole == EAnimGroupRole::TransitionFollower) && !bHasBeenFullWeight)
		{
			GroupNameToUse = NAME_None;
		}

		// Do not use sync groups.
		if (GroupNameToUse == NAME_None && MethodToUse == EAnimSyncMethod::SyncGroup)
		{
			MethodToUse = EAnimSyncMethod::DoNotSync;
		}
	}

	return UE::Anim::FAnimSyncParams(GroupNameToUse, SyncGroupRole, MethodToUse, bOverridePositionWhenJoiningSyncGroupAsLeader);
}
