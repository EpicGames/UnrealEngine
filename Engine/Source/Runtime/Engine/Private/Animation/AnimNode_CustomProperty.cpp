// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_CustomProperty.h"
#include "Animation/AnimInstance.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CustomProperty)

FAnimNode_CustomProperty::FAnimNode_CustomProperty()
	: FAnimNode_Base()
	, TargetInstance(nullptr)
{
}

FAnimNode_CustomProperty::FAnimNode_CustomProperty(const FAnimNode_CustomProperty&) = default;
FAnimNode_CustomProperty::~FAnimNode_CustomProperty() = default;

void FAnimNode_CustomProperty::SetTargetInstance(UObject* InInstance)
{
	TargetInstance = InInstance;
#if ANIMNODE_STATS_VERBOSE
	InitializeStatID();
#endif
}

void FAnimNode_CustomProperty::PropagateInputProperties(const UObject* InSourceInstance)
{
	if(TargetInstance)
	{
		// First copy properties
		check(SourceProperties.Num() == DestProperties.Num());
		for(int32 PropIdx = 0; PropIdx < SourceProperties.Num(); ++PropIdx)
		{
			FProperty* CallerProperty = SourceProperties[PropIdx];
			FProperty* SubProperty = DestProperties[PropIdx];

			if(CallerProperty && SubProperty)
			{
#if WITH_EDITOR
				if (ensure(CallerProperty->SameType(SubProperty)))
#endif
				{
					const uint8* SrcPtr = CallerProperty->ContainerPtrToValuePtr<uint8>(InSourceInstance);
					uint8* DestPtr = SubProperty->ContainerPtrToValuePtr<uint8>(TargetInstance);

					FBoolProperty* CallerBoolProperty = CastField<FBoolProperty>(CallerProperty);
					FBoolProperty* SubBoolProperty = CastField<FBoolProperty>(SubProperty);
					if (CallerBoolProperty && SubBoolProperty)
					{
						// Bools may be native bitfields, in which case we cant call CopyCompleteValue as 
						// there is insufficent information to correctly copy the relevant bits. Calling CopyCompleteValue
						// in this case can potentially overwrite adjacent memory.
						const bool bValue = CallerBoolProperty->GetPropertyValue_InContainer(InSourceInstance);
						SubBoolProperty->SetPropertyValue_InContainer(TargetInstance, bValue);
					}
					else
					{
						CallerProperty->CopyCompleteValue(DestPtr, SrcPtr);
					}
				}
			}
		}
	}
}

void FAnimNode_CustomProperty::InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass)
{
	if(InTargetClass)
	{
		UClass* SourceClass = InSourceInstance->GetClass();

		// Build property lists
		SourceProperties.Reset(SourcePropertyNames.Num());
		DestProperties.Reset(SourcePropertyNames.Num());

		check(SourcePropertyNames.Num() == DestPropertyNames.Num());

		for(int32 Idx = 0; Idx < SourcePropertyNames.Num(); ++Idx)
		{
			const FName& DestName = DestPropertyNames[Idx];

			if (FProperty* DestProperty = FindFProperty<FProperty>(InTargetClass, DestName))
			{
				const FName& SourceName = SourcePropertyNames[Idx];
				FProperty* SourceProperty = FindFProperty<FProperty>(SourceClass, SourceName);

				if (SourceProperty
#if WITH_EDITOR
					// This type check can fail when anim blueprints are in an error state:
					&& SourceProperty->SameType(DestProperty)
#endif
					)
				{
					SourceProperties.Add(SourceProperty);
					DestProperties.Add(DestProperty);
				}
			}
		}
	}
}

#if WITH_EDITOR

void FAnimNode_CustomProperty::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	SourceInstance = const_cast<UAnimInstance*>(InAnimInstance);
}

void FAnimNode_CustomProperty::HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	static IConsoleVariable* UseLegacyAnimInstanceReinstancingBehavior = IConsoleManager::Get().FindConsoleVariable(TEXT("bp.UseLegacyAnimInstanceReinstancingBehavior"));
	if(UseLegacyAnimInstanceReinstancingBehavior == nullptr || !UseLegacyAnimInstanceReinstancingBehavior->GetBool())
	{
		if(InSourceObject)
		{
			InitializeProperties(CastChecked<UAnimInstance>(InSourceObject), GetTargetClass());
		}
	}
}

void FAnimNode_CustomProperty::HandleObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	static IConsoleVariable* UseLegacyAnimInstanceReinstancingBehavior = IConsoleManager::Get().FindConsoleVariable(TEXT("bp.UseLegacyAnimInstanceReinstancingBehavior"));
	if(UseLegacyAnimInstanceReinstancingBehavior == nullptr || !UseLegacyAnimInstanceReinstancingBehavior->GetBool())
	{
		UObject* TargetObject = GetTargetInstance<UObject>();
		UObject* SourceObject = SourceInstance;

		if(TargetObject && SourceObject)
		{
			bool bRelevantObjectWasReinstanced = false;
			for(const TPair<UObject*, UObject*>& ObjectPair : OldToNewInstanceMap)
			{
				if(ObjectPair.Value == TargetObject || ObjectPair.Value == SourceObject)
				{
					bRelevantObjectWasReinstanced = true;
					break;
				}
			}

			if(bRelevantObjectWasReinstanced)
			{
				HandleObjectsReinstanced_Impl(SourceObject, TargetObject, OldToNewInstanceMap);
			}
		}
	}
}

#endif	// #if WITH_EDITOR

