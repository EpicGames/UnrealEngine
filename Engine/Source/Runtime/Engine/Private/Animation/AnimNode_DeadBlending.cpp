// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_DeadBlending.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimNode_SaveCachedPose.h"
#include "Animation/AnimTrace.h"
#include "Animation/BlendProfile.h"
#include "Algo/MaxElement.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Logging/TokenizedMessage.h"
#include "Animation/AnimCurveUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_DeadBlending)

#define LOCTEXT_NAMESPACE "AnimNode_DeadBlending"

TAutoConsoleVariable<int32> CVarAnimDeadBlendingEnable(TEXT("a.AnimNode.DeadBlending.Enable"), 1, TEXT("Enable / Disable DeadBlending"));

namespace UE::Anim
{
	// Inertialization request event bound to a node
	class FDeadBlendingRequester : public IInertializationRequester
	{
	public:
		FDeadBlendingRequester(const FAnimationBaseContext& InContext, FAnimNode_DeadBlending* InNode)
			: Node(*InNode)
			, NodeId(InContext.GetCurrentNodeId())
			, Proxy(*InContext.AnimInstanceProxy)
		{}

	private:
		// IInertializationRequester interface
		virtual void RequestInertialization(
			float InRequestedDuration,
			const UBlendProfile* InBlendProfile) override
		{
			Node.RequestInertialization(FInertializationRequest(InRequestedDuration, InBlendProfile));
		}

		virtual void RequestInertialization(const FInertializationRequest& Request) override
		{
			Node.RequestInertialization(Request);
		}

		virtual void AddDebugRecord(const FAnimInstanceProxy& InSourceProxy, int32 InSourceNodeId) override
		{
#if WITH_EDITORONLY_DATA
			Proxy.RecordNodeAttribute(InSourceProxy, NodeId, InSourceNodeId, IInertializationRequester::Attribute);
#endif
			TRACE_ANIM_NODE_ATTRIBUTE(Proxy, InSourceProxy, NodeId, InSourceNodeId, IInertializationRequester::Attribute);
		}

		virtual FName GetTag() const override { return Node.GetTag(); }

		// Node to target
		FAnimNode_DeadBlending& Node;

		// Node index
		int32 NodeId;

		// Proxy currently executing
		FAnimInstanceProxy& Proxy;
	};

}	// namespace UE::Anim

namespace UE::Anim::DeadBlending::Private
{
	static constexpr float Ln2 = 0.69314718056f;

	static int32 GetNumSkeletonBones(const FBoneContainer& BoneContainer)
	{
		const USkeleton* SkeletonAsset = BoneContainer.GetSkeletonAsset();
		check(SkeletonAsset);

		const FReferenceSkeleton& RefSkeleton = SkeletonAsset->GetReferenceSkeleton();
		return RefSkeleton.GetNum();
	}

	static inline FVector3f VectorDivMax(const float V, const FVector3f W, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector3f(
			V / FMath::Max(W.X, Epsilon),
			V / FMath::Max(W.Y, Epsilon),
			V / FMath::Max(W.Z, Epsilon));
	}

	static inline FVector3f VectorDivMax(const FVector3f V, const FVector3f W, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector3f(
			V.X / FMath::Max(W.X, Epsilon),
			V.Y / FMath::Max(W.Y, Epsilon),
			V.Z / FMath::Max(W.Z, Epsilon));
	}

	static inline FVector VectorDivMax(const FVector V, const FVector W, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector(
			V.X / FMath::Max(W.X, Epsilon),
			V.Y / FMath::Max(W.Y, Epsilon),
			V.Z / FMath::Max(W.Z, Epsilon));
	}

	static inline FVector3f VectorInvExpApprox(const FVector3f V)
	{
		return FVector3f(
			FMath::InvExpApprox(V.X),
			FMath::InvExpApprox(V.Y),
			FMath::InvExpApprox(V.Z));
	}

	static inline FVector VectorEerp(const FVector V, const FVector W, const float Alpha, const float Epsilon = UE_SMALL_NUMBER)
	{
		if (FVector::DistSquared(V, W) < Epsilon)
		{
			return FVector(
				FMath::Lerp(FMath::Max(V.X, Epsilon), FMath::Max(W.X, Epsilon), Alpha),
				FMath::Lerp(FMath::Max(V.Y, Epsilon), FMath::Max(W.Y, Epsilon), Alpha),
				FMath::Lerp(FMath::Max(V.Z, Epsilon), FMath::Max(W.Z, Epsilon), Alpha));
		}
		else
		{
			return FVector(
				FMath::Pow(FMath::Max(V.X, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.X, Epsilon), Alpha),
				FMath::Pow(FMath::Max(V.Y, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.Y, Epsilon), Alpha),
				FMath::Pow(FMath::Max(V.Z, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.Z, Epsilon), Alpha));
		}
	}

	static inline FVector VectorExp(const FVector V)
	{
		return FVector(
			FMath::Exp(V.X),
			FMath::Exp(V.Y),
			FMath::Exp(V.Z));
	}

	static inline FVector VectorLogSafe(const FVector V, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector(
			FMath::Loge(FMath::Max(V.X, Epsilon)),
			FMath::Loge(FMath::Max(V.Y, Epsilon)),
			FMath::Loge(FMath::Max(V.Z, Epsilon)));
	}

	static inline FVector ExtrapolateTranslation(
		const FVector Translation,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(Ln2, DecayHalflife, Epsilon);
			return Translation + (FVector)(VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)));
		}
		else
		{
			return Translation;
		}
	}

	static inline FQuat ExtrapolateRotation(
		const FQuat Rotation,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(Ln2, DecayHalflife, Epsilon);
			return FQuat::MakeFromRotationVector((FVector)(VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)))) * Rotation;
		}
		else
		{
			return Rotation;
		}
	}

	static inline FVector ExtrapolateScale(
		const FVector Scale,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(Ln2, DecayHalflife, Epsilon);
			return VectorExp((FVector)(VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)))) * Scale;
		}
		else
		{
			return Scale;
		}
	}

	static inline float ExtrapolateCurve(
		const float Curve,
		const float Velocity,
		const float Time,
		const float DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER)
	{
		if (FMath::Square(Velocity) > Epsilon)
		{
			const float C = Ln2 / FMath::Max(DecayHalflife, Epsilon);
			return Curve + FMath::Max(Velocity / C, Epsilon) * (1.0f - FMath::InvExpApprox(C * Time));
		}
		else
		{
			return Curve;
		}
	}

	static inline float ClipMagnitudeToGreaterThanEpsilon(const float X, const float Epsilon = UE_KINDA_SMALL_NUMBER)
	{
		return
			X >= 0.0f && X <  Epsilon ?  Epsilon :
			X <  0.0f && X > -Epsilon ? -Epsilon : X;
	}

	static inline float ComputeDecayHalfLifeFromDiffAndVelocity(
		const float SrcDstDiff,
		const float SrcVelocity,
		const float HalfLife,
		const float HalfLifeMin,
		const float HalfLifeMax,
		const float Epsilon = UE_KINDA_SMALL_NUMBER)
	{
		// Essentially what this function does is compute a half-life based on the ratio between the velocity vector and
		// the vector from the source to the destination. This is then clamped to some min and max. If the signs are
		// different (i.e. the velocity and the vector from source to destination are in opposite directions) this will
		// produce a negative number that will get clamped to HalfLifeMin. If the signs match, this will produce a large
		// number when the velocity is small and the vector from source to destination is large, and a small number when
		// the velocity is large and the vector from source to destination is small. This will be clamped either way to 
		// be in the range given by HalfLifeMin and HalfLifeMax. Finally, since the velocity can be close to zero we 
		// have to clamp it to always be greater than some given magnitude (preserving the sign).

		return FMath::Clamp(HalfLife * (SrcDstDiff / ClipMagnitudeToGreaterThanEpsilon(SrcVelocity, Epsilon)), HalfLifeMin, HalfLifeMax);
	}

	static inline FVector3f ComputeDecayHalfLifeFromDiffAndVelocity(
		const FVector SrcDstDiff,
		const FVector3f SrcVelocity,
		const float HalfLife,
		const float HalfLifeMin,
		const float HalfLifeMax,
		const float Epsilon = UE_KINDA_SMALL_NUMBER)
	{
		return FVector3f(
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.X, SrcVelocity.X, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.Y, SrcVelocity.Y, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.Z, SrcVelocity.Z, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon));
	}
}

void FAnimNode_DeadBlending::Deactivate()
{
	InertializationState = EInertializationState::Inactive;

	BoneIndices.Empty();

	BoneTranslations.Empty();
	BoneRotations.Empty();
	BoneRotationDirections.Empty();
	BoneScales.Empty();

	BoneTranslationVelocities.Empty();
	BoneRotationVelocities.Empty();
	BoneScaleVelocities.Empty();

	BoneTranslationDecayHalfLives.Empty();
	BoneRotationDecayHalfLives.Empty();
	BoneScaleDecayHalfLives.Empty();

	InertializationDurationPerBone.Empty();
}

void FAnimNode_DeadBlending::InitFrom(
	const FCompactPose& InPose, 
	const FBlendedCurve& InCurves, 
	const UE::Anim::FStackAttributeContainer& InAttributes, 
	const FInertializationSparsePose& SrcPosePrev, 
	const FInertializationSparsePose& SrcPoseCurr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_DeadBlending::InitFrom);

	check(!SrcPosePrev.IsEmpty() && !SrcPoseCurr.IsEmpty());

	const FBoneContainer& BoneContainer = InPose.GetBoneContainer();

	const int32 NumSkeletonBones = UE::Anim::DeadBlending::Private::GetNumSkeletonBones(BoneContainer);

	// Compute the Inertialization Bone Indices which we will use to index into BoneTranslations, BoneRotations, etc

	BoneIndices.Init(INDEX_NONE, NumSkeletonBones);

	int32 NumInertializationBones = 0;

	for (FCompactPoseBoneIndex BoneIndex : InPose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex == INDEX_NONE ||
			SrcPoseCurr.BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE ||
			SrcPosePrev.BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE)
		{
			continue;
		}

		BoneIndices[SkeletonPoseBoneIndex] = NumInertializationBones;
		NumInertializationBones++;
	}

	// Allocate Inertialization Bones

	BoneTranslations.Init(FVector::ZeroVector, NumInertializationBones);
	BoneRotations.Init(FQuat::Identity, NumInertializationBones);
	BoneRotationDirections.Init(FQuat4f::Identity, NumInertializationBones);
	BoneScales.Init(FVector::OneVector, NumInertializationBones);

	BoneTranslationVelocities.Init(FVector3f::ZeroVector, NumInertializationBones);
	BoneRotationVelocities.Init(FVector3f::ZeroVector, NumInertializationBones);
	BoneScaleVelocities.Init(FVector3f::ZeroVector, NumInertializationBones);

	BoneTranslationDecayHalfLives.Init(ExtrapolationHalfLifeMin * FVector3f::OneVector, NumInertializationBones);
	BoneRotationDecayHalfLives.Init(ExtrapolationHalfLifeMin * FVector3f::OneVector, NumInertializationBones);
	BoneScaleDecayHalfLives.Init(ExtrapolationHalfLifeMin * FVector3f::OneVector, NumInertializationBones);

	for (FCompactPoseBoneIndex BoneIndex : InPose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex == INDEX_NONE || 
			SrcPoseCurr.BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE ||
			SrcPosePrev.BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE)
		{
			continue;
		}

		// Get Bone Indices for Inertialization Bone, Prev and Curr Pose Bones

		const int32 InertializationBoneIndex = BoneIndices[SkeletonPoseBoneIndex];
		const int32 CurrPoseBoneIndex = SrcPoseCurr.BoneIndices[SkeletonPoseBoneIndex];
		const int32 PrevPoseBoneIndex = SrcPosePrev.BoneIndices[SkeletonPoseBoneIndex];

		check(InertializationBoneIndex != INDEX_NONE);
		check(CurrPoseBoneIndex != INDEX_NONE);
		check(PrevPoseBoneIndex != INDEX_NONE);

		// Get Source Animation Transform

		const FVector SrcTranslationCurr = SrcPoseCurr.BoneTranslations[CurrPoseBoneIndex];
		const FQuat SrcRotationCurr = SrcPoseCurr.BoneRotations[CurrPoseBoneIndex];
		const FVector SrcScaleCurr = SrcPoseCurr.BoneScales[CurrPoseBoneIndex];

		BoneTranslations[InertializationBoneIndex] = SrcTranslationCurr;
		BoneRotations[InertializationBoneIndex] = SrcRotationCurr;
		BoneScales[InertializationBoneIndex] = SrcScaleCurr;

		if (SrcPoseCurr.DeltaTime > UE_SMALL_NUMBER)
		{
			// Get Source Animation Velocity

			const FVector SrcTranslationPrev = SrcPosePrev.BoneTranslations[PrevPoseBoneIndex];
			const FQuat SrcRotationPrev = SrcPosePrev.BoneRotations[PrevPoseBoneIndex];
			const FVector SrcScalePrev = SrcPosePrev.BoneScales[PrevPoseBoneIndex];

			const FVector TranslationDiff = SrcTranslationCurr - SrcTranslationPrev;
			const FQuat RotationDiff = (SrcRotationCurr * SrcRotationPrev.Inverse()).GetShortestArcWith(FQuat::Identity);

			const FVector ScaleDiffLinear = SrcScaleCurr - SrcScalePrev;
			const FVector ScaleDiffExponential = UE::Anim::DeadBlending::Private::VectorDivMax(SrcScaleCurr, SrcScalePrev);

			BoneTranslationVelocities[InertializationBoneIndex] = (FVector3f)(TranslationDiff / SrcPoseCurr.DeltaTime);
			BoneRotationVelocities[InertializationBoneIndex] = (FVector3f)(RotationDiff.ToRotationVector() / SrcPoseCurr.DeltaTime);
			BoneScaleVelocities[InertializationBoneIndex] = bLinearlyInterpolateScales ?
				(FVector3f)(ScaleDiffLinear / SrcPoseCurr.DeltaTime) :
				(FVector3f)(UE::Anim::DeadBlending::Private::VectorLogSafe(ScaleDiffExponential) / SrcPoseCurr.DeltaTime);

			// Clamp Maximum Velocity

			BoneTranslationVelocities[InertializationBoneIndex] = BoneTranslationVelocities[InertializationBoneIndex].GetClampedToMaxSize(MaximumTranslationVelocity);
			BoneRotationVelocities[InertializationBoneIndex] = BoneRotationVelocities[InertializationBoneIndex].GetClampedToMaxSize(FMath::DegreesToRadians(MaximumRotationVelocity));
			BoneScaleVelocities[InertializationBoneIndex] = BoneScaleVelocities[InertializationBoneIndex].GetClampedToMaxSize(MaximumScaleVelocity);

			// Compute Decay HalfLives

			const FTransform DstTransform = InPose[BoneIndex];

			const FVector TranslationSrcDstDiff = DstTransform.GetTranslation() - SrcTranslationCurr;

			FQuat RotationSrcDstDiff = DstTransform.GetRotation() * SrcRotationCurr.Inverse();
			RotationSrcDstDiff.EnforceShortestArcWith(FQuat::Identity);

			const FVector ScaleSrcDstDiff = UE::Anim::DeadBlending::Private::VectorDivMax(DstTransform.GetScale3D(), SrcScaleCurr);

			BoneTranslationDecayHalfLives[InertializationBoneIndex] = UE::Anim::DeadBlending::Private::ComputeDecayHalfLifeFromDiffAndVelocity(
				TranslationSrcDstDiff,
				BoneTranslationVelocities[InertializationBoneIndex],
				ExtrapolationHalfLife,
				ExtrapolationHalfLifeMin,
				ExtrapolationHalfLifeMax);

			BoneRotationDecayHalfLives[InertializationBoneIndex] = UE::Anim::DeadBlending::Private::ComputeDecayHalfLifeFromDiffAndVelocity(
				RotationSrcDstDiff.ToRotationVector(),
				BoneRotationVelocities[InertializationBoneIndex],
				ExtrapolationHalfLife,
				ExtrapolationHalfLifeMin,
				ExtrapolationHalfLifeMax);

			BoneScaleDecayHalfLives[InertializationBoneIndex] = UE::Anim::DeadBlending::Private::ComputeDecayHalfLifeFromDiffAndVelocity(
				ScaleSrcDstDiff,
				BoneScaleVelocities[InertializationBoneIndex],
				ExtrapolationHalfLife,
				ExtrapolationHalfLifeMin,
				ExtrapolationHalfLifeMax);
		}
	}

	CurveData.CopyFrom(SrcPoseCurr.Curves.BlendedCurve);

	// Record Source Animation Curve State

	UE::Anim::FNamedValueArrayUtils::Union(CurveData, SrcPoseCurr.Curves.BlendedCurve,
		[this](FDeadBlendingCurveElement& OutResultElement, const UE::Anim::FCurveElement& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			OutResultElement.Value = InElement1.Value;
			OutResultElement.Velocity = 0.0f;
			OutResultElement.HalfLife = ExtrapolationHalfLifeMin;
		});

	// Record Source Animation Curve Velocity

	if (SrcPoseCurr.DeltaTime > UE_SMALL_NUMBER)
	{
		UE::Anim::FNamedValueArrayUtils::Union(CurveData, SrcPosePrev.Curves.BlendedCurve,
			[this, SrcDeltaTime = SrcPoseCurr.DeltaTime](FDeadBlendingCurveElement& OutResultElement, const UE::Anim::FCurveElement& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				bool bSrcCurrValid = (bool)(InFlags & UE::Anim::ENamedValueUnionFlags::ValidArg0);
				bool bSrcPrevValid = (bool)(InFlags & UE::Anim::ENamedValueUnionFlags::ValidArg1);

				if (bSrcCurrValid && bSrcPrevValid)
				{
					OutResultElement.Velocity = (OutResultElement.Value - InElement1.Value) / SrcDeltaTime;
					OutResultElement.Velocity = FMath::Clamp(OutResultElement.Velocity, -MaximumCurveVelocity, MaximumCurveVelocity);
				}
			});
	}

	// Perform Union with Curves from Destination Animation and compute Half-life

	UE::Anim::FNamedValueArrayUtils::Union(CurveData, InCurves,
		[this](FDeadBlendingCurveElement& OutResultElement, const UE::Anim::FCurveElement& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			OutResultElement.HalfLife = UE::Anim::DeadBlending::Private::ComputeDecayHalfLifeFromDiffAndVelocity(
				InElement1.Value - OutResultElement.Value,
				OutResultElement.Velocity,
				ExtrapolationHalfLife,
				ExtrapolationHalfLifeMin,
				ExtrapolationHalfLifeMax);
		});

	// Apply filtering to remove filtered curves from extrapolation. This does not actually
	// prevent these curves from being blended, but does stop them appearing as empty
	// in the output curves created by the Union in ApplyTo unless they are already in the
	// destination animation.
	if (CurveFilter.Num() > 0)
	{
		UE::Anim::FCurveUtils::Filter(CurveData, CurveFilter);
	}

	// Apply filtering to remove curves that are not meant to be extrapolated
	if (ExtrapolatedCurveFilter.Num() > 0)
	{
		UE::Anim::FCurveUtils::Filter(CurveData, ExtrapolatedCurveFilter);
	}

	// Record Root Motion Delta

	// We don't compute the root acceleration difference which can be quite noisy and unreliable
	// and so can cause the extrapolated root velocity to be quite bad and not very useful when
	// being blended out.

	RootTranslationVelocity = FVector3f::ZeroVector;
	RootRotationVelocity = FVector3f::ZeroVector;
	RootScaleVelocity = FVector3f::ZeroVector;

	if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
	{
		FTransform CurrRootMotionDelta = FTransform::Identity;

		if (RootMotionProvider->ExtractRootMotion(InAttributes, CurrRootMotionDelta) &&
			SrcPoseCurr.bHasRootMotion &&
			SrcPoseCurr.DeltaTime > UE_KINDA_SMALL_NUMBER)
		{
			RootTranslationVelocity = (FVector3f)SrcPoseCurr.RootMotionDelta.GetTranslation() / SrcPoseCurr.DeltaTime;
			RootRotationVelocity = (FVector3f)SrcPoseCurr.RootMotionDelta.GetRotation().ToRotationVector() / SrcPoseCurr.DeltaTime;
			RootScaleVelocity = bLinearlyInterpolateScales ?
				((FVector3f)SrcPoseCurr.RootMotionDelta.GetScale3D() / SrcPoseCurr.DeltaTime) :
				((FVector3f)UE::Anim::DeadBlending::Private::VectorLogSafe(SrcPoseCurr.RootMotionDelta.GetScale3D()) / SrcPoseCurr.DeltaTime);
		}
	}
}

void FAnimNode_DeadBlending::ApplyTo(FCompactPose& InOutPose, FBlendedCurve& InOutCurves, UE::Anim::FStackAttributeContainer& InOutAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_DeadBlending::ApplyTo);

	const FBoneContainer& BoneContainer = InOutPose.GetBoneContainer();

	for (FCompactPoseBoneIndex BoneIndex : InOutPose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex == INDEX_NONE || BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE || BoneFilter.Contains(BoneIndex))
		{
			continue;
		}

		const int32 InertializationBoneIndex = BoneIndices[SkeletonPoseBoneIndex];
		check(InertializationBoneIndex != INDEX_NONE);

		// Compute Extrapolated Bone State

		const FVector ExtrapolatedTranslation = UE::Anim::DeadBlending::Private::ExtrapolateTranslation(
			BoneTranslations[InertializationBoneIndex],
			BoneTranslationVelocities[InertializationBoneIndex],
			InertializationTime,
			BoneTranslationDecayHalfLives[InertializationBoneIndex]);

		const FQuat ExtrapolatedRotation = UE::Anim::DeadBlending::Private::ExtrapolateRotation(
			BoneRotations[InertializationBoneIndex],
			BoneRotationVelocities[InertializationBoneIndex],
			InertializationTime,
			BoneRotationDecayHalfLives[InertializationBoneIndex]);

		FVector ExtrapolatedScale = FVector::OneVector;

		if (bLinearlyInterpolateScales)
		{
			// If we are handling scales linearly then treat them like a normal vector such as a translation.
			ExtrapolatedScale = UE::Anim::DeadBlending::Private::ExtrapolateTranslation(
				BoneScales[InertializationBoneIndex],
				BoneScaleVelocities[InertializationBoneIndex],
				InertializationTime,
				BoneScaleDecayHalfLives[InertializationBoneIndex]);
		}
		else
		{
			// Otherwise extrapolate using the exponential version of scalar velocities.
			ExtrapolatedScale = UE::Anim::DeadBlending::Private::ExtrapolateScale(
				BoneScales[InertializationBoneIndex],
				BoneScaleVelocities[InertializationBoneIndex],
				InertializationTime,
				BoneScaleDecayHalfLives[InertializationBoneIndex]);
		}

#if WITH_EDITORONLY_DATA
		if (bShowExtrapolations)
		{
			InOutPose[BoneIndex].SetTranslation(ExtrapolatedTranslation);
			InOutPose[BoneIndex].SetRotation(ExtrapolatedRotation);
			InOutPose[BoneIndex].SetScale3D(ExtrapolatedScale);
			continue;
		}
#endif
		// We need to enforce that the blend of the rotation doesn't suddenly "switch sides"
		// given that the extrapolated rotation can become quite far from the destination
		// animation. To do this we keep track of the blend "direction" and ensure that the
		// delta we are applying to the destination animation always remains on the same
		// side of this rotation.

		FQuat RotationDiff = ExtrapolatedRotation * InOutPose[BoneIndex].GetRotation().Inverse();
		RotationDiff.EnforceShortestArcWith((FQuat)BoneRotationDirections[InertializationBoneIndex]);

		// Update BoneRotationDirections to match our current path
		BoneRotationDirections[InertializationBoneIndex] = (FQuat4f)RotationDiff;

		// Compute Blend Alpha

		const float Alpha = 1.0f - FAlphaBlend::AlphaToBlendOption(
			FMath::Clamp(InertializationTime / FMath::Max(InertializationDurationPerBone[SkeletonPoseBoneIndex], UE_SMALL_NUMBER), 0.0f, 1.0f),
			InertializationBlendMode, InertializationCustomBlendCurve);

		// Perform Blend

		if (Alpha != 0.0f)
		{
			InOutPose[BoneIndex].SetTranslation(FMath::Lerp(InOutPose[BoneIndex].GetTranslation(), ExtrapolatedTranslation, Alpha));
			InOutPose[BoneIndex].SetRotation(FQuat::MakeFromRotationVector(RotationDiff.ToRotationVector() * Alpha) * InOutPose[BoneIndex].GetRotation());

			// Here we use `Eerp` rather than `Lerp` to interpolate scales by default (see: https://theorangeduck.com/page/scalar-velocity).
			// This default is inconsistent with the rest of Unreal which (mostly) uses `Lerp` on scales. The decision 
			// to use `Eerp` by default here is partially due to the fact we are also providing the option of dealing properly 
			// with scalar velocities in this node, and partially to try and not to lock this node into having the same less 
			// accurate behavior by default. Users still have the option to interpolate scales with `Lerp` if they want using bLinearlyInterpolateScales.
			if (bLinearlyInterpolateScales)
			{
				InOutPose[BoneIndex].SetScale3D(FMath::Lerp(InOutPose[BoneIndex].GetScale3D(), ExtrapolatedScale, Alpha));
			}
			else
			{
				InOutPose[BoneIndex].SetScale3D(UE::Anim::DeadBlending::Private::VectorEerp(InOutPose[BoneIndex].GetScale3D(), ExtrapolatedScale, Alpha));
			}
		}
	}

	// Compute Blend Alpha

	const float CurveAlpha = 1.0f - FAlphaBlend::AlphaToBlendOption(
		FMath::Clamp(InertializationTime / FMath::Max(InertializationDuration, UE_SMALL_NUMBER), 0.0f, 1.0f),
		InertializationBlendMode, InertializationCustomBlendCurve);

	// Blend Curves

	PoseCurveData.CopyFrom(InOutCurves);

	UE::Anim::FNamedValueArrayUtils::Union(InOutCurves, PoseCurveData, CurveData, [CurveAlpha, this](
			UE::Anim::FCurveElement& OutResultElement, 
			const UE::Anim::FCurveElement& InElement0,
			const FDeadBlendingCurveElement& InElement1, 
			UE::Anim::ENamedValueUnionFlags InFlags)
		{
			// For filtered Curves take destination value

			if (FilteredCurves.Contains(OutResultElement.Name))
			{
				OutResultElement.Value = InElement0.Value;
				OutResultElement.Flags = InElement0.Flags;
				return;
			}

			// Compute Extrapolated Curve Value

			const float ExtrapolatedCurve = UE::Anim::DeadBlending::Private::ExtrapolateCurve(
				InElement1.Value,
				InElement1.Velocity,
				InertializationTime,
				InElement1.HalfLife);

#if WITH_EDITORONLY_DATA
			if (bShowExtrapolations)
			{
				OutResultElement.Value = ExtrapolatedCurve;
				OutResultElement.Flags = InElement0.Flags | InElement1.Flags;
				return;
			}
#endif

			OutResultElement.Value = FMath::Lerp(InElement0.Value, ExtrapolatedCurve, CurveAlpha);
			OutResultElement.Flags = InElement0.Flags | InElement1.Flags;
		});

	// Blend Root Motion Delta

	if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
	{
		// Compute Blend Alpha

		const float Alpha = 1.0f - FAlphaBlend::AlphaToBlendOption(
			FMath::Clamp(InertializationTime / FMath::Max(InertializationDurationPerBone[0], UE_SMALL_NUMBER), 0.0f, 1.0f),
			InertializationBlendMode, InertializationCustomBlendCurve);

		if (Alpha != 0.0f)
		{
			FTransform CurrRootMotionDelta = FTransform::Identity;
			if (RootMotionProvider->ExtractRootMotion(InOutAttributes, CurrRootMotionDelta))
			{
				CurrRootMotionDelta.SetTranslation(FMath::Lerp(CurrRootMotionDelta.GetTranslation(), DeltaTime * (FVector)RootTranslationVelocity, Alpha));
				CurrRootMotionDelta.SetRotation(FQuat::MakeFromRotationVector(DeltaTime * (FVector)RootRotationVelocity * Alpha) * CurrRootMotionDelta.GetRotation());

				if (bLinearlyInterpolateScales)
				{
					CurrRootMotionDelta.SetScale3D(FMath::Lerp(CurrRootMotionDelta.GetScale3D(), DeltaTime * (FVector)RootScaleVelocity, Alpha));
				}
				else
				{
					CurrRootMotionDelta.SetScale3D(UE::Anim::DeadBlending::Private::VectorEerp(CurrRootMotionDelta.GetScale3D(), DeltaTime * (FVector)RootScaleVelocity, Alpha));
				}

				RootMotionProvider->OverrideRootMotion(CurrRootMotionDelta, InOutAttributes);
			}
		}
	}
}

class USkeleton* FAnimNode_DeadBlending::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	bInvalidSkeletonIsError = false;

	if (const IAnimClassInterface* AnimClassInterface = GetAnimClassInterface())
	{
		return AnimClassInterface->GetTargetSkeleton();
	}

	return nullptr;
}

FAnimNode_DeadBlending::FAnimNode_DeadBlending() {}

void FAnimNode_DeadBlending::RequestInertialization(const FInertializationRequest& Request)
{
	if (Request.Duration >= 0.0f)
	{
		RequestQueue.AddUnique(Request);
	}
}

void FAnimNode_DeadBlending::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread);

	FAnimNode_Base::Initialize_AnyThread(Context);
	Source.Initialize(Context);

	CurveFilter.Empty();
	CurveFilter.SetFilterMode(UE::Anim::ECurveFilterMode::DisallowFiltered);
	CurveFilter.AppendNames(FilteredCurves);

	ExtrapolatedCurveFilter.Empty();
	ExtrapolatedCurveFilter.SetFilterMode(UE::Anim::ECurveFilterMode::DisallowFiltered);
	ExtrapolatedCurveFilter.AppendNames(ExtrapolationFilteredCurves);

	BoneFilter.Init(FCompactPoseBoneIndex(INDEX_NONE), FilteredBones.Num());
	
	PrevPoseSnapshot.Empty();
	CurrPoseSnapshot.Empty();

	RequestQueue.Reserve(8);

	BoneIndices.Empty();
	
	BoneTranslations.Empty();
	BoneRotations.Empty();
	BoneRotationDirections.Empty();
	BoneScales.Empty();

	BoneTranslationVelocities.Empty();
	BoneRotationVelocities.Empty();
	BoneScaleVelocities.Empty();

	BoneTranslationDecayHalfLives.Empty();
	BoneRotationDecayHalfLives.Empty();
	BoneScaleDecayHalfLives.Empty();

	CurveData.Empty();

	DeltaTime = 0.0f;

	InertializationState = EInertializationState::Inactive;
	InertializationTime = 0.0f;

	InertializationDuration = 0.0f;
	InertializationDurationPerBone.Empty();
	InertializationMaxDuration = 0.0f;

	InertializationBlendMode = DefaultBlendMode;
	InertializationCustomBlendCurve = DefaultCustomBlendCurve;
}


void FAnimNode_DeadBlending::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread);

	FAnimNode_Base::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);

	// Compute Compact Pose Bone Index for each bone in Filter

	const FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();
	BoneFilter.Init(FCompactPoseBoneIndex(INDEX_NONE), FilteredBones.Num());
	for (int32 FilterBoneIdx = 0; FilterBoneIdx < FilteredBones.Num(); FilterBoneIdx++)
	{
		FilteredBones[FilterBoneIdx].Initialize(Context.AnimInstanceProxy->GetSkeleton());
		BoneFilter[FilterBoneIdx] = FilteredBones[FilterBoneIdx].GetCompactPoseIndex(RequiredBones);
	}
}


void FAnimNode_DeadBlending::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread);

	const bool bNeedsReset =
		bResetOnBecomingRelevant &&
		UpdateCounter.HasEverBeenUpdated() &&
		!UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter());

	if (bNeedsReset)
	{
		// Clear any pending inertialization requests
		RequestQueue.Reset();

		// Clear the inertialization state
		Deactivate();

		// Clear the pose history
		PrevPoseSnapshot.Empty();
		CurrPoseSnapshot.Empty();
	}

	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	// Catch the inertialization request message and call the node's RequestInertialization function with the request
	UE::Anim::TScopedGraphMessage<UE::Anim::FDeadBlendingRequester> InertializationMessage(Context, Context, this);

	if (bForwardRequestsThroughSkippedCachedPoseNodes)
	{
		const int32 NodeId = Context.GetCurrentNodeId();
		const FAnimInstanceProxy& Proxy = *Context.AnimInstanceProxy;

		// Handle skipped updates for cached poses by forwarding to inertialization nodes in those residual stacks
		UE::Anim::TScopedGraphMessage<UE::Anim::FCachedPoseSkippedUpdateHandler> CachedPoseSkippedUpdate(Context, [this, NodeId, &Proxy](TArrayView<const UE::Anim::FMessageStack> InSkippedUpdates)
		{
			// If we have a pending request forward the request to other Inertialization nodes
			// that were skipped due to pose caching.
			if (RequestQueue.Num() > 0)
			{
				// Cached poses have their Update function called once even though there may be multiple UseCachedPose nodes for the same pose.
				// Because of this, there may be Inertialization ancestors of the UseCachedPose nodes that missed out on requests.
				// So here we forward 'this' node's requests to the ancestors of those skipped UseCachedPose nodes.
				// Note that in some cases, we may be forwarding the requests back to this same node.  Those duplicate requests will ultimately
				// be ignored by the 'AddUnique' in the body of FAnimNode_DeadBlending::RequestInertialization.
				for (const UE::Anim::FMessageStack& Stack : InSkippedUpdates)
				{
					Stack.ForEachMessage<UE::Anim::IInertializationRequester>([this, NodeId, &Proxy](UE::Anim::IInertializationRequester& InMessage)
					{
						for (const FInertializationRequest& Request : RequestQueue)
						{
							InMessage.RequestInertialization(Request);
						}
						InMessage.AddDebugRecord(Proxy, NodeId);

						return UE::Anim::FMessageStack::EEnumerate::Stop;
					});
				}
			}
		});

		// Context message stack lifetime is scope based so we need to call Source.Update() before exiting the scope of the message above.
		Source.Update(Context);
	}
	else
	{
		Source.Update(Context);
	}

	// Accumulate delta time between calls to Evaluate_AnyThread
	DeltaTime += Context.GetDeltaTime();
}

void FAnimNode_DeadBlending::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread);

	// Evaluate the Input and write it to the Output

	Source.Evaluate(Output);

	// Disable inertialization if requested (for testing / debugging)
	if (!CVarAnimDeadBlendingEnable.GetValueOnAnyThread())
	{
		// Clear any pending inertialization requests
		RequestQueue.Reset();

		// Clear the inertialization state
		Deactivate();

		// Clear the pose history
		PrevPoseSnapshot.Empty();
		CurrPoseSnapshot.Empty();

		// Reset the cached time accumulator
		DeltaTime = 0.0f;

		return;
	}

	// Automatically detect teleports... note that we do the teleport distance check against the root bone's location (world space) rather
	// than the mesh component's location because we still want to inertialize instances where the skeletal mesh component has been moved
	// while simultaneously counter-moving the root bone (as is the case when mounting and dismounting vehicles for example)

	const FTransform ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();

	bool bTeleported = false;

	const float TeleportDistanceThreshold = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetTeleportDistanceThreshold();

	if (!CurrPoseSnapshot.IsEmpty() && TeleportDistanceThreshold > 0.0f)
	{
		const FVector RootWorldSpaceLocation = ComponentTransform.TransformPosition(Output.Pose[FCompactPoseBoneIndex(0)].GetTranslation());
		
		const int32 RootBoneIndex = CurrPoseSnapshot.BoneIndices[0];

		if (RootBoneIndex != INDEX_NONE)
		{
			const FVector PrevRootWorldSpaceLocation = CurrPoseSnapshot.ComponentTransform.TransformPosition(CurrPoseSnapshot.BoneTranslations[RootBoneIndex]);

			if (FVector::DistSquared(RootWorldSpaceLocation, PrevRootWorldSpaceLocation) > FMath::Square(TeleportDistanceThreshold))
			{
				bTeleported = true;
			}
		}
	}

	// If teleported we simply reset the inertialization

	if (bTeleported)
	{
		Deactivate();
	}
	
	// If we don't have any Pose Snapshots recorded it means this is the first time this node has been evaluated in 
	// which case there shouldn't be any discontinuity to remove, so no inertialization needs to be done, and we can 
	// discard any requests.

	if (CurrPoseSnapshot.IsEmpty() && PrevPoseSnapshot.IsEmpty())
	{
		RequestQueue.Reset();
	}

	// Process Inertialization Requests

	if (!RequestQueue.IsEmpty())
	{

		// Find shortest request
		int32 ShortestRequestIdx = INDEX_NONE;
		float ShortestRequestDuration = UE_MAX_FLT;
		for (int32 RequestIdx = 0; RequestIdx < RequestQueue.Num(); RequestIdx++)
		{
			const FInertializationRequest& Request = RequestQueue[RequestIdx];
			
			// If the request has a specific tag, it needs to match ours
			if (Request.Tag == NAME_None || Request.Tag == GetTag())
			{
				if (Request.Duration < ShortestRequestDuration)
				{
					ShortestRequestIdx = RequestIdx;
					ShortestRequestDuration = Request.Duration;
				}
			}
		}

		if (ShortestRequestIdx != INDEX_NONE)
		{
			const int32 NumSkeletonBones = UE::Anim::DeadBlending::Private::GetNumSkeletonBones(Output.AnimInstanceProxy->GetRequiredBones());

			// Record Request

			InertializationTime = 0.0f;
			InertializationDuration = BlendTimeMultiplier * RequestQueue[ShortestRequestIdx].Duration;
			InertializationDurationPerBone.Init(InertializationDuration, NumSkeletonBones);

			const USkeleton* TargetSkeleton = Output.AnimInstanceProxy->GetRequiredBones().GetSkeletonAsset();
			if (RequestQueue[ShortestRequestIdx].BlendProfile)
			{
				RequestQueue[ShortestRequestIdx].BlendProfile->FillSkeletonBoneDurationsArray(InertializationDurationPerBone, InertializationDuration, TargetSkeleton);
			}
			else if (DefaultBlendProfile)
			{
				DefaultBlendProfile->FillSkeletonBoneDurationsArray(InertializationDurationPerBone, InertializationDuration, TargetSkeleton);
			}

			// Cache the maximum duration across all bones (so we know when to deactivate the inertialization request)
			InertializationMaxDuration = FMath::Max(InertializationDuration, *Algo::MaxElement(InertializationDurationPerBone));

			if (RequestQueue[ShortestRequestIdx].bUseBlendMode)
			{
				InertializationBlendMode = RequestQueue[ShortestRequestIdx].BlendMode;
				InertializationCustomBlendCurve = RequestQueue[ShortestRequestIdx].CustomBlendCurve;
			}

#if ANIM_TRACE_ENABLED
			InertializationRequestDescription = RequestQueue[ShortestRequestIdx].DescriptionString;
			InertializationRequestNodeId = RequestQueue[ShortestRequestIdx].NodeId;
			InertializationRequestAnimInstance = RequestQueue[ShortestRequestIdx].AnimInstance;
#endif

			// Override with defaults

			if (bAlwaysUseDefaultBlendSettings)
			{
				InertializationDuration = BlendTimeMultiplier * DefaultBlendDuration;
				InertializationDurationPerBone.Init(BlendTimeMultiplier * DefaultBlendDuration, NumSkeletonBones);
				InertializationMaxDuration = BlendTimeMultiplier * DefaultBlendDuration;
				InertializationBlendMode = DefaultBlendMode;
				InertializationCustomBlendCurve = DefaultCustomBlendCurve;
			}

			check(InertializationDuration != UE_MAX_FLT);
			check(InertializationMaxDuration != UE_MAX_FLT);

			// Initialize the recorded pose state at the point of transition

			if (!PrevPoseSnapshot.IsEmpty() && !CurrPoseSnapshot.IsEmpty())
			{
				// We have two previous poses and so can initialize as normal.

				InitFrom(
					Output.Pose,
					Output.Curve,
					Output.CustomAttributes,
					PrevPoseSnapshot,
					CurrPoseSnapshot);
			}
			else if (!CurrPoseSnapshot.IsEmpty())
			{
				// We only have a single previous pose. Repeat this pose assuming zero velocity.

				InitFrom(
					Output.Pose,
					Output.Curve,
					Output.CustomAttributes,
					CurrPoseSnapshot,
					CurrPoseSnapshot);
			}
			else
			{
				// This should never happen because we are not able to issue an inertialization 
				// requested until we have at least one pose recorded in the snapshots.
				check(false);
			}

			// Set state to active

			InertializationState = EInertializationState::Active;
		}
		// Reset Request Queue
		RequestQueue.Reset();
	}

	// Update Time Since Transition and deactivate if blend is over

	if (InertializationState == EInertializationState::Active)
	{
		InertializationTime += DeltaTime;

		if (InertializationTime >= InertializationMaxDuration)
		{
			Deactivate();
		}
	}

	// Apply inertialization

	if (InertializationState == EInertializationState::Active)
	{
		ApplyTo(Output.Pose, Output.Curve, Output.CustomAttributes);
	}

	// Find AttachParentName

	FName AttachParentName = NAME_None;
	if (AActor* Owner = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetOwner())
	{
		if (AActor* AttachParentActor = Owner->GetAttachParentActor())
		{
			AttachParentName = AttachParentActor->GetFName();
		}
	}

	// Record Pose Snapshot

	if (!CurrPoseSnapshot.IsEmpty())
	{
		// Directly swap the memory of the current pose with the prev pose snapshot (to avoid allocations and copies)
		Swap(PrevPoseSnapshot, CurrPoseSnapshot);
	}
	
	// Initialize the current pose
	CurrPoseSnapshot.InitFrom(Output.Pose, Output.Curve, Output.CustomAttributes, ComponentTransform, AttachParentName, DeltaTime);

	// Reset Delta Time

	DeltaTime = 0.0f;

	const float InertializationWeight = InertializationState == EInertializationState::Active ?
		1.0f - FAlphaBlend::AlphaToBlendOption(
			FMath::Clamp(InertializationTime / FMath::Max(InertializationDuration, UE_SMALL_NUMBER), 0.0f, 1.0f),
			InertializationBlendMode, InertializationCustomBlendCurve) : 0.0f;

	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("State"), *UEnum::GetValueAsString(InertializationState));
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Elapsed Time"), InertializationTime);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Duration"), InertializationDuration);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Max Duration"), InertializationMaxDuration);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Normalized Time"), InertializationDuration > UE_KINDA_SMALL_NUMBER ? (InertializationTime / InertializationDuration) : 0.0f);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Inertialization Weight"), InertializationWeight);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Request Description"), *InertializationRequestDescription);
	TRACE_ANIM_NODE_VALUE_WITH_ID_ANIM_NODE(Output, GetNodeIndex(), TEXT("Request Node"), InertializationRequestNodeId, InertializationRequestAnimInstance);

	TRACE_ANIM_INERTIALIZATION(*Output.AnimInstanceProxy, GetNodeIndex(), InertializationWeight, FAnimTrace::EInertializationType::DeadBlending);
}

void FAnimNode_DeadBlending::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)

	FString DebugLine = DebugData.GetNodeName(this);
	
	float InertializationWeight = InertializationState == EInertializationState::Active ?
		1.0f - FAlphaBlend::AlphaToBlendOption(
			FMath::Clamp(InertializationTime / FMath::Max(InertializationDuration, UE_SMALL_NUMBER), 0.0f, 1.0f),
			InertializationBlendMode, InertializationCustomBlendCurve) : 0.0f;
	
	if (InertializationDuration > UE_KINDA_SMALL_NUMBER)
	{
		DebugLine += FString::Printf(TEXT("('%s' Time: %.3f / %.3f (%.0f%%) Weight: %.3f"),
			*UEnum::GetValueAsString(InertializationState),
			InertializationTime,
			InertializationDuration,
			100.0f * InertializationTime / InertializationDuration,
			InertializationWeight);
	}
	else
	{
		DebugLine += FString::Printf(TEXT("('%s' Time: %.3f / %.3f) Weight: %.3f"),
			*UEnum::GetValueAsString(InertializationState),
			InertializationTime,
			InertializationDuration,
			InertializationWeight);
	}

	DebugData.AddDebugItem(DebugLine);
	Source.GatherDebugData(DebugData);
}

bool FAnimNode_DeadBlending::NeedsDynamicReset() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
