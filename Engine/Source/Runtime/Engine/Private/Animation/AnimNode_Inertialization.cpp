// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimNode_SaveCachedPose.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "Animation/BlendProfile.h"
#include "Algo/MaxElement.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Logging/TokenizedMessage.h"
#include "Animation/AnimCurveUtils.h"
#include "Misc/UObjectToken.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Inertialization)

#define LOCTEXT_NAMESPACE "AnimNode_Inertialization"

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::Anim::IInertializationRequester);

const FName UE::Anim::IInertializationRequester::Attribute("InertialBlending");

TAutoConsoleVariable<int32> CVarAnimInertializationEnable(TEXT("a.AnimNode.Inertialization.Enable"), 1, TEXT("Enable / Disable Inertialization"));
TAutoConsoleVariable<int32> CVarAnimInertializationIgnoreVelocity(TEXT("a.AnimNode.Inertialization.IgnoreVelocity"), 0, TEXT("Ignore velocity information during Inertialization (effectively reverting to a quintic diff blend)"));
TAutoConsoleVariable<int32> CVarAnimInertializationIgnoreDeficit(TEXT("a.AnimNode.Inertialization.IgnoreDeficit"), 0, TEXT("Ignore inertialization time deficit caused by interruptions"));


namespace UE::Anim
{

	void IInertializationRequester::RequestInertialization(const FInertializationRequest& InInertializationRequest)
	{
		RequestInertialization(InInertializationRequest.Duration, InInertializationRequest.BlendProfile);
	}

	// Inertialization request event bound to a node
	class FInertializationRequester : public IInertializationRequester
	{
	public:
		FInertializationRequester(const FAnimationBaseContext& InContext, FAnimNode_Inertialization* InNode)
			: Node(*InNode)
			, NodeId(InContext.GetCurrentNodeId())
			, Proxy(*InContext.AnimInstanceProxy)
		{}

	private:
		// IInertializationRequester interface
		virtual void RequestInertialization(float InRequestedDuration, const UBlendProfile* InBlendProfile) override
		{
			Node.RequestInertialization(InRequestedDuration, InBlendProfile);
		}

		virtual void RequestInertialization(const FInertializationRequest& InInertializationRequest) override
		{
			// The Blend Mode parameters will be ignored as FAnimNode_Inertialization does not support them.
			Node.RequestInertialization(InInertializationRequest);
		}

		virtual void AddDebugRecord(const FAnimInstanceProxy& InSourceProxy, int32 InSourceNodeId) override
		{
#if WITH_EDITORONLY_DATA
			Proxy.RecordNodeAttribute(InSourceProxy, NodeId, InSourceNodeId, IInertializationRequester::Attribute);
#endif
			TRACE_ANIM_NODE_ATTRIBUTE(Proxy, InSourceProxy, NodeId, InSourceNodeId, IInertializationRequester::Attribute);
		}

		virtual FName GetTag() const override { return Node.GetTag(); }

		// Node to target
		FAnimNode_Inertialization& Node;

		// Node index
		int32 NodeId;

		// Proxy currently executing
		FAnimInstanceProxy& Proxy;
	};
}

namespace UE::Anim::Inertialization::Private
{
	static inline int32 GetNumSkeletonBones(const FBoneContainer& BoneContainer)
	{
		const USkeleton* SkeletonAsset = BoneContainer.GetSkeletonAsset();
		check(SkeletonAsset);

		const FReferenceSkeleton& RefSkeleton = SkeletonAsset->GetReferenceSkeleton();
		return RefSkeleton.GetNum();
	}

	static constexpr float INERTIALIZATION_TIME_EPSILON = 1.0e-7f;

	// Calculate the "inertialized" value of a single float at time t
	//
	// @param x0	Initial value of the float (at time 0)
	// @param v0	Initial "velocity" (first derivative) of the float (at time 0)
	// @param t		Time at which to evaluate the float
	// @param t1	Ending inertialization time (ie: the time at which the curve must be zero)
	//
	// Evaluates a quintic polynomial curve with the specified initial conditions (x0, v0) which hits zero at time t1.  As well,
	// the curve is designed so that the first and second derivatives are also zero at time t1.
	//
	// The initial second derivative (a0) is chosen such that it is as close to zero as possible, but large enough to prevent any
	// overshoot (ie: enforce x >= 0 for t between 0 and t1).  If necessary, the ending time t1 will be adjusted (shortened) to
	// guarantee that there is no overshoot, even for very large initial velocities.
	//
	static float CalcInertialFloat(float x0, float v0, float t, float t1)
	{
		static_assert(INERTIALIZATION_TIME_EPSILON * INERTIALIZATION_TIME_EPSILON * INERTIALIZATION_TIME_EPSILON * INERTIALIZATION_TIME_EPSILON * INERTIALIZATION_TIME_EPSILON > FLT_MIN,
			"INERTIALIZATION_TIME_EPSILON^5 must be greater than FLT_MIN to avoid denormalization (and potential division by zero) for very small values of t1");

		if (t < 0.0f)
		{
			t = 0.0f;
		}

		if (t >= t1 - INERTIALIZATION_TIME_EPSILON)
		{
			return 0.0f;
		}

		// Assume that x0 >= 0... if this is not the case, then simply invert everything (both input and output)
		float sign = 1.0f;
		if (x0 < 0.0f)
		{
			x0 = -x0;
			v0 = -v0;
			sign = -1.0f;
		}

		// If v0 > 0, then the curve will overshoot away from zero, so clamp v0 here to guarantee that there is no overshoot
		if (v0 > 0.0f)
		{
			v0 = 0.0f;
		}

		// Check for invalid values - this is only expected to occur if NaNs or other invalid values are coming into the node
		if (!ensureMsgf(x0 >= 0.0f && v0 <= 0.0f && t >= 0.0f && t1 >= 0.0f,
			TEXT("Invalid Value(s) in Inertialization - x0: %f, v0: %f, t: %f, t1: %f"), x0, v0, t, t1))
		{
			x0 = 0.0f;
			v0 = 0.0f;
			t = 0.0f;
			t1 = 0.0f;
		}

		// Limit t1 such that the curve does not overshoot below zero (ensuring that x >= 0 for all t between 0 and t1).
		//
		// We observe that if the curve does overshoot below zero, it must have an inflection point somewhere between 0 and t1
		// (since we know that both x0 and x1 are >= 0).  Therefore, we can prevent overshoot by adjusting t1 such that any
		// inflection point is at t >= t1.
		//
		// Assuming that we are using the zero jerk version of the curve (see below) whose velocity has a triple root at t1, then
		// we can prevent overshoot by forcing the remaining root to be at time t >= t1, or equivalently, we can set t1 to be the
		// lesser of the original t1 or the value that gives us a solution with a quadruple velocity root at t1.
		//
		// The following Mathematica expression solves for t1 that gives us the quadruple velocity root:
		//
		//		v := q * (t-t1)^4
		//		x := Integrate[Expand[v], t] + x0
		//		eq1 := (v/.t->0)==v0
		//		eq2 := (x/.t->t1)==0
		//		Solve[{eq1 && eq2}, {q,t1}]
		//
		if (v0 < -UE_KINDA_SMALL_NUMBER)
		{
			t1 = FMath::Min(t1, -5.0f * x0 / v0);
		}

		if (t >= t1 - INERTIALIZATION_TIME_EPSILON)
		{
			return 0.0f;
		}

		const float t1_2 = t1 * t1;
		const float t1_3 = t1 * t1_2;
		const float t1_4 = t1 * t1_3;
		const float t1_5 = t1 * t1_4;

		// Compute the initial acceleration value (a0) for this curve.  Ideally we want to use an initial acceleration of zero, but
		// if there is a large negative initial velocity, then we will need to use a larger acceleration in order to ensure that
		// the curve does not overshoot below zero (ie: to ensure that x >= 0 for all t between 0 and t1).
		//
		// To compute a0, we first compute the a0 that we would get if we also specified that the third derivative (the "jerk" j)
		// is zero at t1.  If this value of a0 is positive (and therefore opposing the initial velocity), then we use that.  If it
		// is negative, then we simply use an initial a0 of zero.
		//
		// The following Mathematica expression solves for a0 that gives us zero jerk at t1:
		//
		//		x:= A*t^5 + B*t^4 + C*t^3 + D*t^2 + v0*t + x0
		//		v:=Dt[x, t, Constants->{A,B,C,D,v0,x0}]
		//		a:=Dt[v, t, Constants->{A,B,C,D,v0,x0}]
		//		j:=Dt[a, t, Constants->{A,B,C,D,v0,x0}]
		//		eq1:=(x/.t->t1)==0
		//		eq2:=(v/.t->t1)==0
		//		eq3:=(a/.t->t1)==0
		//		eq4:=(j/.t->t1)==0
		//		a0:=a/.t->0/.Solve[{eq1 && eq2 && eq3 && eq4}, {A,B,C,D}]
		//		ExpandNumerator[a0]
		//
		const float a0 = FMath::Max(0.0f, (-8.0f * t1 * v0 - 20.0f * x0) / t1_2);

		// Compute the polynomial coefficients given the starting and ending conditions, solved from:
		//
		//		x:= A*t^5 + B*t^4 + C*t^3 + D*t^2 + v0*t + x0
		//		v:=Dt[x, t, Constants->{A,B,C,D,v0,x0}]
		//		a:=Dt[v, t, Constants->{A,B,C,D,v0,x0}]
		//		eq1:=(x/.t->t1)==0
		//		eq2:=(v/.t->t1)==0
		//		eq3:=(a/.t->t1)==0
		//		eq4:=(a/.t->0)==a0
		//		Simplify[Solve[{eq1 && eq2 && eq3 && eq4}, {A,B,C,D}]]
		//
		const float A = -0.5f * (a0 * t1_2 + 6.0f * t1 * v0 + 12.0f * x0) / t1_5;
		const float B = 0.5f * (3.0f * a0 * t1_2 + 16.0f * t1 * v0 + 30.0f * x0) / t1_4;
		const float C = -0.5f * (3.0f * a0 * t1_2 + 12.0f * t1 * v0 + 20.0f * x0) / t1_3;
		const float D = 0.5f * a0;

		const float x = (((((A * t) + B) * t + C) * t + D) * t + v0) * t + x0;

		return x * sign;
	}

}	// namespace UE::Anim::Inertialization::Private

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FInertializationRequest::FInertializationRequest() = default;

FInertializationRequest::FInertializationRequest(float InDuration, const UBlendProfile* InBlendProfile)
	: Duration(InDuration)
	, BlendProfile(InBlendProfile) {}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FInertializationRequest::Clear()
{
	Duration = -1.0f;
	BlendProfile = nullptr;
	bUseBlendMode = false;
	BlendMode = EAlphaBlendOption::Linear;
	CustomBlendCurve = nullptr;

#if ANIM_TRACE_ENABLED 
	DescriptionString.Empty();
	NodeId = INDEX_NONE;
	AnimInstance = nullptr;
#endif
}

class USkeleton* FAnimNode_Inertialization::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	bInvalidSkeletonIsError = false;

	if (const IAnimClassInterface* AnimClassInterface = GetAnimClassInterface())
	{
		return AnimClassInterface->GetTargetSkeleton();
	}

	return nullptr;
}

void FAnimNode_Inertialization::RequestInertialization(float Duration, const UBlendProfile* BlendProfile)
{
	if (Duration >= 0.0f)
	{
		RequestQueue.AddUnique(FInertializationRequest(Duration, BlendProfile));
	}
}

void FAnimNode_Inertialization::RequestInertialization(const FInertializationRequest& Request)
{
	if (Request.Duration >= 0.0f)
	{
		RequestQueue.AddUnique(Request);
	}
}

/*static*/ void FAnimNode_Inertialization::LogRequestError(const FAnimationUpdateContext& Context, const int32 NodePropertyIndex)
{
#if WITH_EDITORONLY_DATA	
	UAnimBlueprint* AnimBlueprint = Context.AnimInstanceProxy->GetAnimBlueprint();
	UAnimBlueprintGeneratedClass* AnimClass = AnimBlueprint ? AnimBlueprint->GetAnimBlueprintGeneratedClass() : nullptr;
	const UObject* RequesterNode = AnimClass ? AnimClass->GetVisualNodeFromNodePropertyIndex(NodePropertyIndex) : nullptr;

	Context.LogMessage(FTokenizedMessage::Create(EMessageSeverity::Error)
		->AddToken(FTextToken::Create(LOCTEXT("InertializationRequestError_1", "No Inertialization node found for request from ")))
		->AddToken(FUObjectToken::Create(RequesterNode))
		->AddToken(FTextToken::Create(LOCTEXT("InertializationRequestError_2", ". Add an Inertialization node after this request."))));
#endif
}

/*static*/ void FAnimNode_Inertialization::LogRequestError(const FAnimationUpdateContext& Context, const FPoseLinkBase& RequesterPoseLink)
{
#if WITH_EDITORONLY_DATA	
	LogRequestError(Context, RequesterPoseLink.SourceLinkID);
#endif
}

void FAnimNode_Inertialization::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread);

	FAnimNode_Base::Initialize_AnyThread(Context);
	Source.Initialize(Context);

	CurveFilter.Empty();
	CurveFilter.SetFilterMode(UE::Anim::ECurveFilterMode::DisallowFiltered);
	CurveFilter.AppendNames(FilteredCurves);

	BoneFilter.Init(FCompactPoseBoneIndex(INDEX_NONE), FilteredBones.Num());

	PrevPoseSnapshot.Empty();
	CurrPoseSnapshot.Empty();

	RequestQueue.Reserve(8);

	BoneIndices.Empty();

	BoneTranslationDiffDirection.Empty();
	BoneTranslationDiffMagnitude.Empty();
	BoneTranslationDiffSpeed.Empty();

	BoneRotationDiffAxis.Empty();
	BoneRotationDiffAngle.Empty();
	BoneRotationDiffSpeed.Empty();

	BoneScaleDiffAxis.Empty();
	BoneScaleDiffMagnitude.Empty();
	BoneScaleDiffSpeed.Empty();

	CurveDiffs.Empty();

	DeltaTime = 0.0f;

	InertializationState = EInertializationState::Inactive;
	InertializationElapsedTime = 0.0f;

	InertializationDuration = 0.0f;
	InertializationDurationPerBone.Empty();
	InertializationMaxDuration = 0.0f;

	InertializationDeficit = 0.0f;
}


void FAnimNode_Inertialization::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread);

	FAnimNode_Base::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);

	// Compute Compact Pose Bone Index for each bone in Filter

	const FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();
	BoneFilter.Init(FCompactPoseBoneIndex(INDEX_NONE), FilteredBones.Num());
	for (int32 FilterBoneIdx = 0; FilterBoneIdx < FilteredBones.Num(); FilterBoneIdx++)
	{
		FilteredBones[FilterBoneIdx].Initialize(Context.AnimInstanceProxy->GetSkeleton());
		BoneFilter[FilterBoneIdx] = FilteredBones[FilterBoneIdx].GetCompactPoseIndex(RequiredBones);
	}
}


void FAnimNode_Inertialization::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread);

	const bool bNeedsReset =
		bResetOnBecomingRelevant &&
		UpdateCounter.HasEverBeenUpdated() &&
		!UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter());

	if (bNeedsReset)
	{
		// Clear any pending inertialization requests
		RequestQueue.Reset();

		// Clear the inertialization state
		Deactivate();

		// Clear the pose history
		PrevPoseSnapshot.Empty();
		CurrPoseSnapshot.Empty();
	}

	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	// Catch the inertialization request message and call the node's RequestInertialization function with the request
	UE::Anim::TScopedGraphMessage<UE::Anim::FInertializationRequester> InertializationMessage(Context, Context, this);

	if (bForwardRequestsThroughSkippedCachedPoseNodes)
	{
		const int32 NodeId = Context.GetCurrentNodeId();
		const FAnimInstanceProxy& Proxy = *Context.AnimInstanceProxy;

		// Handle skipped updates for cached poses by forwarding to inertialization nodes in those residual stacks
		UE::Anim::TScopedGraphMessage<UE::Anim::FCachedPoseSkippedUpdateHandler> CachedPoseSkippedUpdate(Context, [this, NodeId, &Proxy](TArrayView<const UE::Anim::FMessageStack> InSkippedUpdates)
		{
			// If we have a pending request forward the request to other Inertialization nodes
			// that were skipped due to pose caching.
			if (RequestQueue.Num() > 0)
			{
				// Cached poses have their Update function called once even though there may be multiple UseCachedPose nodes for the same pose.
				// Because of this, there may be Inertialization ancestors of the UseCachedPose nodes that missed out on requests.
				// So here we forward 'this' node's requests to the ancestors of those skipped UseCachedPose nodes.
				// Note that in some cases, we may be forwarding the requests back to this same node.  Those duplicate requests will ultimately
				// be ignored by the 'AddUnique' in the body of FAnimNode_Inertialization::RequestInertialization.
				for (const UE::Anim::FMessageStack& Stack : InSkippedUpdates)
				{
					Stack.ForEachMessage<UE::Anim::IInertializationRequester>([this, NodeId, &Proxy](UE::Anim::IInertializationRequester& InMessage)
					{
						for (const FInertializationRequest& Request : RequestQueue)
						{
							InMessage.RequestInertialization(Request);
						}
						InMessage.AddDebugRecord(Proxy, NodeId);

						return UE::Anim::FMessageStack::EEnumerate::Stop;
					});
				}
			}
		});

		// Context message stack lifetime is scope based so we need to call Source.Update() before exiting the scope of the message above.
		Source.Update(Context);
	}
	else
	{
		Source.Update(Context);
	}

	// Accumulate delta time between calls to Evaluate_AnyThread
	DeltaTime += Context.GetDeltaTime();
}

void FAnimNode_Inertialization::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread);
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(Inertialization, !IsInGameThread());

	Source.Evaluate(Output);

	// Disable inertialization if requested (for testing / debugging)
	if (!CVarAnimInertializationEnable.GetValueOnAnyThread())
	{
		// Clear any pending inertialization requests
		RequestQueue.Reset();

		// Clear the inertialization state
		Deactivate();

		// Clear the pose history
		PrevPoseSnapshot.Empty();
		CurrPoseSnapshot.Empty();

		// Reset the cached time accumulator
		DeltaTime = 0.0f;

		return;
	}

	// Filter requests with tags that do not match ours
	for (int32 RequestIndex = RequestQueue.Num()-1; RequestIndex >= 0; --RequestIndex)
	{
		const FInertializationRequest& Request = RequestQueue[RequestIndex];
		if (Request.Tag != NAME_None && Request.Tag != GetTag())
		{
			RequestQueue.RemoveAtSwap(RequestIndex);
		}
	}

	// Update the inertialization state if a new inertialization request is pending
	const int32 NumRequests = RequestQueue.Num();
	if (NumRequests > 0 && !CurrPoseSnapshot.IsEmpty())
	{
		float AppliedDeficit = 0.0f;
		if (InertializationState == EInertializationState::Active)
		{
			// An active inertialization is being interrupted. Keep track of the lost inertialization time
			// and reduce future durations if interruptions continue. Without this mitigation,
			// repeated interruptions will lead to a degenerate pose because the pose target is unstable.
			bool bApplyDeficit = InertializationDeficit > 0.0f && !CVarAnimInertializationIgnoreDeficit.GetValueOnAnyThread();
			InertializationDeficit = InertializationDuration - InertializationElapsedTime;
			AppliedDeficit = bApplyDeficit ? InertializationDeficit : 0.0f;
		}

		InertializationState = EInertializationState::Pending;
		InertializationElapsedTime = 0.0f;
		
		const int32 NumSkeletonBones = UE::Anim::Inertialization::Private::GetNumSkeletonBones(Output.AnimInstanceProxy->GetRequiredBones());

		const USkeleton* TargetSkeleton = Output.AnimInstanceProxy->GetRequiredBones().GetSkeletonAsset();
		auto FillSkeletonBoneDurationsArray = [this, NumSkeletonBones, TargetSkeleton](auto& DurationPerBone, float Duration, const UBlendProfile* BlendProfile) {
			if (BlendProfile == nullptr)
			{
				BlendProfile = DefaultBlendProfile;
			}

			if (BlendProfile != nullptr)
			{
				DurationPerBone.SetNum(NumSkeletonBones);
				BlendProfile->FillSkeletonBoneDurationsArray(DurationPerBone, Duration, TargetSkeleton);
			}
			else
			{
				DurationPerBone.Init(Duration, NumSkeletonBones);
			}
		};

		// Handle the first inertialization request in the queue
		InertializationDuration = FMath::Max(RequestQueue[0].Duration - AppliedDeficit, 0.0f);
#if ANIM_TRACE_ENABLED
		InertializationRequestDescription = RequestQueue[0].DescriptionString;
		InertializationRequestNodeId = RequestQueue[0].NodeId;
		InertializationRequestAnimInstance = RequestQueue[0].AnimInstance;
#endif

		FillSkeletonBoneDurationsArray(InertializationDurationPerBone, InertializationDuration, RequestQueue[0].BlendProfile);

		// Handle all subsequent inertialization requests (often there will be only a single request)
		if (NumRequests > 1)
		{
			UE::Anim::TTypedIndexArray<FSkeletonPoseBoneIndex, float, FAnimStackAllocator> RequestDurationPerBone;
			for (int32 RequestIndex = 1; RequestIndex < NumRequests; ++RequestIndex)
			{
				const FInertializationRequest& Request = RequestQueue[RequestIndex];
				const float RequestDuration = FMath::Max(Request.Duration - AppliedDeficit, 0.0f);

				// Merge this request in with the previous requests (using the minimum requested time per bone)
				if (RequestDuration < InertializationDuration)
				{
					InertializationDuration = RequestDuration;
#if ANIM_TRACE_ENABLED
					InertializationRequestDescription = Request.DescriptionString;
					InertializationRequestNodeId = Request.NodeId;
					InertializationRequestAnimInstance = Request.AnimInstance;
#endif
				}

				if (Request.BlendProfile != nullptr)
				{
					FillSkeletonBoneDurationsArray(RequestDurationPerBone, RequestDuration, Request.BlendProfile);
					for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < NumSkeletonBones; ++SkeletonBoneIndex)
					{
						InertializationDurationPerBone[SkeletonBoneIndex] = FMath::Min(InertializationDurationPerBone[SkeletonBoneIndex], RequestDurationPerBone[SkeletonBoneIndex]);
					}
				}
				else
				{
					for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < NumSkeletonBones; ++SkeletonBoneIndex)
					{
						InertializationDurationPerBone[SkeletonBoneIndex] = FMath::Min(InertializationDurationPerBone[SkeletonBoneIndex], RequestDuration);
					}
				}
			}
		}

		// Cache the maximum duration across all bones (so we know when to deactivate the inertialization request)
		InertializationMaxDuration = FMath::Max(InertializationDuration, *Algo::MaxElement(InertializationDurationPerBone));
	}

	RequestQueue.Reset();

	// Update the inertialization timer
	if (InertializationState != EInertializationState::Inactive)
	{
		InertializationElapsedTime += DeltaTime;
		if (InertializationElapsedTime >= InertializationDuration)
		{
			// Reset the deficit accumulator
			InertializationDeficit = 0.0f;
		}
		else
		{
			// Pay down the accumulated deficit caused by interruptions
			InertializationDeficit -= FMath::Min(InertializationDeficit, DeltaTime);
		}

		if (InertializationElapsedTime >= InertializationMaxDuration)
		{
			Deactivate();
		}
	}

	const FTransform ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();

	// Automatically detect teleports... note that we do the teleport distance check against the root bone's location (world space) rather
	// than the mesh component's location because we still want to inertialize instances where the skeletal mesh component has been moved
	// while simultaneously counter-moving the root bone (as is the case when mounting and dismounting vehicles for example)

	bool bTeleported = false;
	const float TeleportDistanceThreshold = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetTeleportDistanceThreshold();
	if (!bTeleported && !CurrPoseSnapshot.IsEmpty() && TeleportDistanceThreshold > 0.0f)
	{
		const FVector RootWorldSpaceLocation = ComponentTransform.TransformPosition(Output.Pose[FCompactPoseBoneIndex(0)].GetTranslation());

		const int32 RootBoneIndex = CurrPoseSnapshot.BoneIndices[0];

		if (RootBoneIndex != INDEX_NONE)
		{
			const FVector PrevRootWorldSpaceLocation = CurrPoseSnapshot.ComponentTransform.TransformPosition(CurrPoseSnapshot.BoneTranslations[RootBoneIndex]);

			if (FVector::DistSquared(RootWorldSpaceLocation, PrevRootWorldSpaceLocation) > FMath::Square(TeleportDistanceThreshold))
			{
				bTeleported = true;
			}
		}
	}

	if (bTeleported)
	{
		// Cancel inertialization requests during teleports
		if (InertializationState == EInertializationState::Pending)
		{
			Deactivate();
		}

		// Clear the time accumulator during teleports (in order to invalidate any recorded velocities during the teleport)
		DeltaTime = 0.0f;
	}

	// Ignore the inertialization velocities if requested (for testing / debugging)
	if (CVarAnimInertializationIgnoreVelocity.GetValueOnAnyThread())
	{
		// Clear the time accumulator (so as to invalidate any recorded velocities)
		DeltaTime = 0.0f;
	}

	// Get the parent actor attachment information (to detect and counteract discontinuities when changing parents)
	FName AttachParentName = NAME_None;
	if (AActor* Owner = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetOwner())
	{
		if (AActor* AttachParentActor = Owner->GetAttachParentActor())
		{
			AttachParentName = AttachParentActor->GetFName();
		}
	}

	// Inertialize the pose

	if (InertializationState == EInertializationState::Pending)
	{
		if (!PrevPoseSnapshot.IsEmpty() && !CurrPoseSnapshot.IsEmpty())
		{
			// We have two previous poses and so can record the offset as normal.

			InitFrom(
				Output.Pose,
				Output.Curve,
				Output.CustomAttributes,
				ComponentTransform,
				AttachParentName,
				CurrPoseSnapshot,
				PrevPoseSnapshot);
		}
		else if (!CurrPoseSnapshot.IsEmpty())
		{
			// We only have a single previous pose. Repeat this pose (assuming zero velocity).

			InitFrom(
				Output.Pose,
				Output.Curve,
				Output.CustomAttributes,
				ComponentTransform,
				AttachParentName,
				CurrPoseSnapshot,
				CurrPoseSnapshot);
		}
		else
		{
			// This should never happen because we are not able to issue an inertialization 
			// requested until we have at least one pose recorded in the snapshots.
			check(false);
		}

		InertializationState = EInertializationState::Active;
	}

	// Apply the inertialization offset

	if (InertializationState == EInertializationState::Active)
	{
		ApplyTo(Output.Pose, Output.Curve, Output.CustomAttributes);
	}

	// Record Pose Snapshot

	if (!CurrPoseSnapshot.IsEmpty())
	{
		// Directly swap the memory of the current pose with the prev pose snapshot (to avoid allocations and copies)
		Swap(PrevPoseSnapshot, CurrPoseSnapshot);
	}
	
	// Initialize the current pose
	CurrPoseSnapshot.InitFrom(Output.Pose, Output.Curve, Output.CustomAttributes, ComponentTransform, AttachParentName, DeltaTime);
	

	// Reset the time accumulator and teleport state
	DeltaTime = 0.0f;

	const float NormalizedInertializationTime = InertializationDuration > UE_KINDA_SMALL_NUMBER ? (InertializationElapsedTime / InertializationDuration) : 0.0f;
	const float InertializationWeight = InertializationState == EInertializationState::Active ? 1.0f - NormalizedInertializationTime : 0.0f;

	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("State"), *UEnum::GetValueAsString(InertializationState));
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Elapsed Time"), InertializationElapsedTime);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Duration"), InertializationDuration);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Max Duration"), InertializationMaxDuration);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Normalized Time"), InertializationDuration > UE_KINDA_SMALL_NUMBER ? (InertializationElapsedTime / InertializationDuration) : 0.0f);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Inertialization Weight"), InertializationWeight);
	TRACE_ANIM_NODE_VALUE_WITH_ID(Output, GetNodeIndex(), TEXT("Request Description"), *InertializationRequestDescription);
	TRACE_ANIM_NODE_VALUE_WITH_ID_ANIM_NODE(Output, GetNodeIndex(), TEXT("Request Node"), InertializationRequestNodeId, InertializationRequestAnimInstance);

	TRACE_ANIM_INERTIALIZATION(*Output.AnimInstanceProxy, GetNodeIndex(), InertializationWeight, FAnimTrace::EInertializationType::Inertialization);
}


void FAnimNode_Inertialization::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData);

	FString DebugLine = DebugData.GetNodeName(this);

	if (InertializationDuration > UE_KINDA_SMALL_NUMBER)
	{
		DebugLine += FString::Printf(TEXT("('%s' Time: %.3f / %.3f (%.0f%%) [%.3f])"),
			*UEnum::GetValueAsString(InertializationState),
			InertializationElapsedTime,
			InertializationDuration,
			100.0f * InertializationElapsedTime / InertializationDuration,
			InertializationDeficit);
	}
	else
	{
		DebugLine += FString::Printf(TEXT("('%s' Time: %.3f / %.3f [%.3f])"),
			*UEnum::GetValueAsString(InertializationState),
			InertializationElapsedTime,
			InertializationDuration,
			InertializationDeficit);
	}
	DebugData.AddDebugItem(DebugLine);

	Source.GatherDebugData(DebugData);
}


bool FAnimNode_Inertialization::NeedsDynamicReset() const
{
	return true;
}


void FAnimNode_Inertialization::ResetDynamics(ETeleportType InTeleportType)
{
	// Note: InTeleportType is unused and teleports are detected automatically (UE-78594)
}

void FAnimNode_Inertialization::InitFrom(
	const FCompactPose& InPose, 
	const FBlendedCurve& InCurves, 
	const UE::Anim::FStackAttributeContainer& InAttributes,
	const FTransform& ComponentTransform, 
	const FName AttachParentName, 
	const FInertializationSparsePose& Prev1, 
	const FInertializationSparsePose& Prev2)
{	
	check(!Prev1.IsEmpty() && !Prev2.IsEmpty());

	// Compute the Inertialization Space

	const FQuat ComponentTransformGetRotationInverse = ComponentTransform.GetRotation().Inverse();

	// Determine if we should initialize in local space (the default) or in world space (for situations where we wish to correct
	// a world-space discontinuity such as an abrupt orientation change)
	EInertializationSpace InertializationSpace = EInertializationSpace::Default;
	if (AttachParentName != Prev1.AttachParentName || AttachParentName != Prev2.AttachParentName)
	{
		// If the parent space has changed, then inertialize in world space
		InertializationSpace = EInertializationSpace::WorldSpace;
	}
	else if (AttachParentName == NAME_None)
	{
		// If there was a discontinuity in ComponentTransform orientation, then correct for that by inertializing the orientation in world space
		// (but only if the mesh is not attached to another actor, because we don't want to dampen the connection between attached actors)
		if ((FMath::Abs((Prev1.ComponentTransform.GetRotation() * ComponentTransformGetRotationInverse).W) < 0.999f) ||	// (W < 0.999f --> angle > 5 degrees)
			(FMath::Abs((Prev2.ComponentTransform.GetRotation() * ComponentTransformGetRotationInverse).W) < 0.999f))	// (W < 0.999f --> angle > 5 degrees)
		{
			InertializationSpace = EInertializationSpace::WorldRotation;
		}
	}

	// Compute the Inertialization Bone Indices which we will use to index into BoneTranslations, BoneRotations, etc

	const FBoneContainer& BoneContainer = InPose.GetBoneContainer();

	const int32 NumSkeletonBones = UE::Anim::Inertialization::Private::GetNumSkeletonBones(BoneContainer);

	BoneIndices.Init(INDEX_NONE, NumSkeletonBones);

	int32 NumInertializationBones = 0;

	for (FCompactPoseBoneIndex BoneIndex : InPose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex == INDEX_NONE || 
			Prev1.BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE ||
			Prev2.BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE)
		{
			continue;
		}

		BoneIndices[SkeletonPoseBoneIndex] = NumInertializationBones;
		NumInertializationBones++;
	}

	// Allocate Inertialization Bones

	BoneTranslationDiffDirection.Init(FVector3f::ZeroVector, NumInertializationBones);
	BoneTranslationDiffMagnitude.Init(0.0f, NumInertializationBones);
	BoneTranslationDiffSpeed.Init(0.0f, NumInertializationBones);
	BoneRotationDiffAxis.Init(FVector3f::ZeroVector, NumInertializationBones);
	BoneRotationDiffAngle.Init(0.0f, NumInertializationBones);
	BoneRotationDiffSpeed.Init(0.0f, NumInertializationBones);
	BoneScaleDiffAxis.Init(FVector3f::ZeroVector, NumInertializationBones);
	BoneScaleDiffMagnitude.Init(0.0f, NumInertializationBones);
	BoneScaleDiffSpeed.Init(0.0f, NumInertializationBones);
	
	// Compute Pose Differences

	for (FCompactPoseBoneIndex BoneIndex : InPose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex == INDEX_NONE ||
			Prev1.BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE ||
			Prev2.BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE)
		{
			continue;
		}

		// Get Bone Indices for Inertialization Bone, Prev and Curr Pose Bones

		const int32 InertializationBoneIndex = BoneIndices[SkeletonPoseBoneIndex];
		const int32 Prev1PoseBoneIndex = Prev1.BoneIndices[SkeletonPoseBoneIndex];
		const int32 Prev2PoseBoneIndex = Prev2.BoneIndices[SkeletonPoseBoneIndex];

		check(InertializationBoneIndex != INDEX_NONE);
		check(Prev1PoseBoneIndex != INDEX_NONE);
		check(Prev2PoseBoneIndex != INDEX_NONE);

		const FTransform PoseTransform = InPose[BoneIndex];
		FTransform Prev1Transform = FTransform(Prev1.BoneRotations[Prev1PoseBoneIndex], Prev1.BoneTranslations[Prev1PoseBoneIndex], Prev1.BoneScales[Prev1PoseBoneIndex]);
		FTransform Prev2Transform = FTransform(Prev2.BoneRotations[Prev2PoseBoneIndex], Prev2.BoneTranslations[Prev2PoseBoneIndex], Prev2.BoneScales[Prev2PoseBoneIndex]);

		if (BoneIndex.IsRootBone())
		{
			// If we are inertializing in world space, then adjust the historical root bones to be in a consistent reference frame
			if (InertializationSpace == EInertializationSpace::WorldSpace)
			{
				Prev1Transform *= Prev1.ComponentTransform.GetRelativeTransform(ComponentTransform);
				Prev2Transform *= Prev2.ComponentTransform.GetRelativeTransform(ComponentTransform);
			}
			else if (InertializationSpace == EInertializationSpace::WorldRotation)
			{
				Prev1Transform.SetRotation(ComponentTransformGetRotationInverse * Prev1.ComponentTransform.GetRotation() * Prev1Transform.GetRotation());
				Prev2Transform.SetRotation(ComponentTransformGetRotationInverse * Prev2.ComponentTransform.GetRotation() * Prev2Transform.GetRotation());
			}
		}

		// Compute the bone translation difference
		{
			FVector TranslationDirection = FVector::ZeroVector;
			float TranslationMagnitude = 0.0f;
			float TranslationSpeed = 0.0f;

			const FVector T = Prev1Transform.GetTranslation() - PoseTransform.GetTranslation();
			TranslationMagnitude = T.Size();
			if (TranslationMagnitude > UE_KINDA_SMALL_NUMBER)
			{
				TranslationDirection = T / TranslationMagnitude;
			}

			if (Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER && TranslationMagnitude > UE_KINDA_SMALL_NUMBER)
			{
				const FVector PrevT = Prev2Transform.GetTranslation() - PoseTransform.GetTranslation();
				const float PrevMagnitude = FVector::DotProduct(PrevT, TranslationDirection);
				TranslationSpeed = (TranslationMagnitude - PrevMagnitude) / Prev1.DeltaTime;
			}

			BoneTranslationDiffDirection[InertializationBoneIndex] = (FVector3f)TranslationDirection;
			BoneTranslationDiffMagnitude[InertializationBoneIndex] = TranslationMagnitude;
			BoneTranslationDiffSpeed[InertializationBoneIndex] = TranslationSpeed;
		}

		// Compute the bone rotation difference
		{
			FVector RotationAxis = FVector::ZeroVector;
			float RotationAngle = 0.0f;
			float RotationSpeed = 0.0f;

			const FQuat Q = Prev1Transform.GetRotation() * PoseTransform.GetRotation().Inverse();
			Q.ToAxisAndAngle(RotationAxis, RotationAngle);
			RotationAngle = FMath::UnwindRadians(RotationAngle);
			if (RotationAngle < 0.0f)
			{
				RotationAxis = -RotationAxis;
				RotationAngle = -RotationAngle;
			}

			if (Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER && RotationAngle > UE_KINDA_SMALL_NUMBER)
			{
				const FQuat PrevQ = Prev2Transform.GetRotation() * PoseTransform.GetRotation().Inverse();
				const float PrevAngle = PrevQ.GetTwistAngle(RotationAxis);
				RotationSpeed = FMath::UnwindRadians(RotationAngle - PrevAngle) / Prev1.DeltaTime;
			}

			BoneRotationDiffAxis[InertializationBoneIndex] = (FVector3f)RotationAxis;
			BoneRotationDiffAngle[InertializationBoneIndex] = RotationAngle;
			BoneRotationDiffSpeed[InertializationBoneIndex] = RotationSpeed;
		}

		// Compute the bone scale difference
		{
			FVector ScaleAxis = FVector::ZeroVector;
			float ScaleMagnitude = 0.0f;
			float ScaleSpeed = 0.0f;

			const FVector S = Prev1Transform.GetScale3D() - PoseTransform.GetScale3D();
			ScaleMagnitude = S.Size();
			if (ScaleMagnitude > UE_KINDA_SMALL_NUMBER)
			{
				ScaleAxis = S / ScaleMagnitude;
			}

			if (Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER && ScaleMagnitude > UE_KINDA_SMALL_NUMBER)
			{
				const FVector PrevS = Prev2Transform.GetScale3D() - PoseTransform.GetScale3D();
				const float PrevMagnitude = FVector::DotProduct(PrevS, ScaleAxis);
				ScaleSpeed = (ScaleMagnitude - PrevMagnitude) / Prev1.DeltaTime;
			}

			BoneScaleDiffAxis[InertializationBoneIndex] = (FVector3f)ScaleAxis;
			BoneScaleDiffMagnitude[InertializationBoneIndex] = ScaleMagnitude;
			BoneScaleDiffSpeed[InertializationBoneIndex] = ScaleSpeed;
		}
	}

	// Compute the curve differences
	// First copy in current values
	CurveDiffs.CopyFrom(InCurves);

	// Compute differences
	UE::Anim::FNamedValueArrayUtils::Union(CurveDiffs, Prev1.Curves.BlendedCurve,
		[](FInertializationCurveDiffElement& OutResultElement, const UE::Anim::FCurveElement& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			OutResultElement.Delta = InElement1.Value - OutResultElement.Value;
		});

	// Compute derivatives
	if (Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER)
	{
		UE::Anim::FNamedValueArrayUtils::Union(CurveDiffs, Prev2.Curves.BlendedCurve,
			[DeltaTime = Prev1.DeltaTime](FInertializationCurveDiffElement& OutResultElement, const UE::Anim::FCurveElement& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				const float Prev1Weight = OutResultElement.Delta - OutResultElement.Value;
				const float Prev2Weight = InElement1.Value;
				OutResultElement.Derivative = (Prev1Weight - Prev2Weight) / DeltaTime;
			});
	}

	// Apply filtering to remove filtered curves from diffs. This does not actually
	// prevent these curves from being inertialized, but does stop them appearing as empty
	// in the output curves created by the Union in ApplyTo unless they are already in the
	// destination animation.
	if (CurveFilter.Num() > 0)
	{
		UE::Anim::FCurveUtils::Filter(CurveDiffs, CurveFilter);
	}

	// Compute Root Motion Delta Difference

	// We don't compute the speed difference since this essentially represents
	// the acceleration of the root motion, which can be quite noisy and unreliable
	// and so can cause the computed offsets to be bad when they are blended out

	RootTranslationVelocityDiffDirection = FVector3f::ZeroVector;
	RootTranslationVelocityDiffMagnitude = 0.0f;
	RootRotationVelocityDiffDirection = FVector3f::ZeroVector;
	RootRotationVelocityDiffMagnitude = 0.0f;
	RootScaleVelocityDiffDirection = FVector3f::ZeroVector;
	RootScaleVelocityDiffMagnitude = 0.0f;

	if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
	{
		FTransform CurrRootMotionDelta = FTransform::Identity;

		if (RootMotionProvider->ExtractRootMotion(InAttributes, CurrRootMotionDelta) && 
			Prev1.bHasRootMotion &&
			Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER && 
			DeltaTime > UE_KINDA_SMALL_NUMBER)
		{
			const FVector RootTranslationVelocityDiff = (Prev1.RootMotionDelta.GetTranslation() / Prev1.DeltaTime) - (CurrRootMotionDelta.GetTranslation() / DeltaTime);
			RootTranslationVelocityDiffMagnitude = RootTranslationVelocityDiff.Size();
			if (RootTranslationVelocityDiffMagnitude > UE_KINDA_SMALL_NUMBER)
			{
				RootTranslationVelocityDiffDirection = (FVector3f)RootTranslationVelocityDiff / RootTranslationVelocityDiffMagnitude;
			}

			const FVector RootRotationVelocityDiff = (Prev1.RootMotionDelta.GetRotation().ToRotationVector() / Prev1.DeltaTime) - (CurrRootMotionDelta.GetRotation().ToRotationVector() / DeltaTime);
			RootRotationVelocityDiffMagnitude = RootRotationVelocityDiff.Size();
			if (RootRotationVelocityDiffMagnitude > UE_KINDA_SMALL_NUMBER)
			{
				RootRotationVelocityDiffDirection = (FVector3f)RootRotationVelocityDiff / RootRotationVelocityDiffMagnitude;
			}

			const FVector RootScaleVelocityDiff = (Prev1.RootMotionDelta.GetScale3D() / Prev1.DeltaTime) - (CurrRootMotionDelta.GetScale3D() / DeltaTime);
			RootScaleVelocityDiffMagnitude = RootScaleVelocityDiff.Size();
			if (RootScaleVelocityDiffMagnitude > UE_KINDA_SMALL_NUMBER)
			{
				RootScaleVelocityDiffDirection = (FVector3f)RootScaleVelocityDiff / RootScaleVelocityDiffMagnitude;
			}
		}
	}
}

void FAnimNode_Inertialization::ApplyTo(FCompactPose& InOutPose, FBlendedCurve& InOutCurves, UE::Anim::FStackAttributeContainer& InOutAttributes)
{
	const FBoneContainer& BoneContainer = InOutPose.GetBoneContainer();

	// Apply pose difference
	for (FCompactPoseBoneIndex BoneIndex : InOutPose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex == INDEX_NONE || BoneIndices[SkeletonPoseBoneIndex] == INDEX_NONE || BoneFilter.Contains(BoneIndex))
		{
			continue;
		}

		const int32 InertializationBoneIndex = BoneIndices[SkeletonPoseBoneIndex];
		check(InertializationBoneIndex != INDEX_NONE);

		const float Duration = InertializationDurationPerBone[SkeletonPoseBoneIndex];

		// Apply the bone translation difference
		const FVector T = (FVector)BoneTranslationDiffDirection[InertializationBoneIndex] *
			UE::Anim::Inertialization::Private::CalcInertialFloat(BoneTranslationDiffMagnitude[InertializationBoneIndex], BoneTranslationDiffSpeed[InertializationBoneIndex], InertializationElapsedTime, Duration);
		InOutPose[BoneIndex].AddToTranslation(T);

		// Apply the bone rotation difference
		const FQuat Q = FQuat((FVector)BoneRotationDiffAxis[InertializationBoneIndex],
			UE::Anim::Inertialization::Private::CalcInertialFloat(BoneRotationDiffAngle[InertializationBoneIndex], BoneRotationDiffSpeed[InertializationBoneIndex], InertializationElapsedTime, Duration));
		InOutPose[BoneIndex].SetRotation(Q * InOutPose[BoneIndex].GetRotation());

		// Apply the bone scale difference
		const FVector S = (FVector)BoneScaleDiffAxis[InertializationBoneIndex] *
			UE::Anim::Inertialization::Private::CalcInertialFloat(BoneScaleDiffMagnitude[InertializationBoneIndex], BoneScaleDiffSpeed[InertializationBoneIndex], InertializationElapsedTime, Duration);
		InOutPose[BoneIndex].SetScale3D(S + InOutPose[BoneIndex].GetScale3D());
	}

	InOutPose.NormalizeRotations();

	// Apply curve differences

	PoseCurveData.CopyFrom(InOutCurves);

	UE::Anim::FNamedValueArrayUtils::Union(InOutCurves, PoseCurveData, CurveDiffs, [this](
		UE::Anim::FCurveElement& OutResultElement, 
		const UE::Anim::FCurveElement& InElement0,
		const FInertializationCurveDiffElement& InElement1,
		UE::Anim::ENamedValueUnionFlags InFlags)
		{
			// For filtered Curves take destination value

			if (FilteredCurves.Contains(OutResultElement.Name))
			{
				OutResultElement.Value = InElement0.Value;
				OutResultElement.Flags = InElement0.Flags;
				return;
			}

			// Otherwise take destination value plus offset

			OutResultElement.Value = InElement0.Value + UE::Anim::Inertialization::Private::CalcInertialFloat(InElement1.Delta, InElement1.Derivative, InertializationElapsedTime, InertializationDuration);
			OutResultElement.Flags = InElement0.Flags | InElement1.Flags;
		});

	// Apply Root Motion Delta Difference

	if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
	{
		FTransform CurrRootMotionDelta = FTransform::Identity;
		if (RootMotionProvider->ExtractRootMotion(InOutAttributes, CurrRootMotionDelta))
		{
			// Use Blend Duration from Root Bone
			const float Duration = InertializationDurationPerBone[0];

			// Apply the root translation velocity difference
			const FVector T = DeltaTime * (FVector)RootTranslationVelocityDiffDirection *
				UE::Anim::Inertialization::Private::CalcInertialFloat(RootTranslationVelocityDiffMagnitude, 0.0f, InertializationElapsedTime, Duration);
			CurrRootMotionDelta.AddToTranslation(T);

			// Apply the root rotation velocity difference
			const FQuat Q = FQuat::MakeFromRotationVector(DeltaTime * (FVector)RootRotationVelocityDiffDirection *
				UE::Anim::Inertialization::Private::CalcInertialFloat(RootRotationVelocityDiffMagnitude, 0.0f, InertializationElapsedTime, Duration));
			CurrRootMotionDelta.SetRotation(Q * CurrRootMotionDelta.GetRotation());

			// Apply the root scale velocity difference
			const FVector S = DeltaTime * (FVector)RootScaleVelocityDiffDirection *
				UE::Anim::Inertialization::Private::CalcInertialFloat(RootScaleVelocityDiffMagnitude, 0.0f, InertializationElapsedTime, Duration);
			CurrRootMotionDelta.SetScale3D(S + CurrRootMotionDelta.GetScale3D());

			RootMotionProvider->OverrideRootMotion(CurrRootMotionDelta, InOutAttributes);
		}
	}
}

void FAnimNode_Inertialization::Deactivate()
{
	InertializationState = EInertializationState::Inactive;

	BoneIndices.Empty();
	
	BoneTranslationDiffDirection.Empty();
	BoneTranslationDiffMagnitude.Empty();
	BoneTranslationDiffSpeed.Empty();
	
	BoneRotationDiffAxis.Empty();
	BoneRotationDiffAngle.Empty();
	BoneRotationDiffSpeed.Empty();
	
	BoneScaleDiffAxis.Empty();
	BoneScaleDiffMagnitude.Empty();
	BoneScaleDiffSpeed.Empty();

	CurveDiffs.Empty();

	InertializationDurationPerBone.Empty();

	InertializationElapsedTime = 0.0f;
	InertializationDuration = 0.0f;
	InertializationMaxDuration = 0.0f;
	InertializationDeficit = 0.0f;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

// DEPRECATED: See FInertializationSparsePose::InitFrom
void FInertializationPose::InitFrom(const FCompactPose& Pose, const FBlendedCurve& InCurves, const FTransform& InComponentTransform, const FName& InAttachParentName, float InDeltaTime)
{
	const FBoneContainer& BoneContainer = Pose.GetBoneContainer();

	const int32 NumSkeletonBones = UE::Anim::Inertialization::Private::GetNumSkeletonBones(BoneContainer);
	BoneTransforms.Reset(NumSkeletonBones);
	BoneTransforms.AddZeroed(NumSkeletonBones);
	BoneStates.Reset(NumSkeletonBones);
	BoneStates.AddZeroed(NumSkeletonBones);
	for (FCompactPoseBoneIndex BoneIndex : Pose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);
		if (SkeletonPoseBoneIndex != INDEX_NONE)
		{
			BoneTransforms[SkeletonPoseBoneIndex] = Pose[BoneIndex];
			BoneStates[SkeletonPoseBoneIndex] = EInertializationBoneState::Valid;
		}
	}

	Curves.InitFrom(InCurves);
	ComponentTransform = InComponentTransform;
	AttachParentName = InAttachParentName;
	DeltaTime = InDeltaTime;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FInertializationSparsePose::InitFrom(
	const FCompactPose& Pose,
	const FBlendedCurve& InCurves,
	const UE::Anim::FStackAttributeContainer& Attributes,
	const FTransform& InComponentTransform,
	const FName InAttachParentName,
	const float InDeltaTime)
{
	const FBoneContainer& BoneContainer = Pose.GetBoneContainer();

	const int32 NumSkeletonBones = UE::Anim::Inertialization::Private::GetNumSkeletonBones(BoneContainer);

	// Allocate Bone Index Array

	BoneIndices.Init(INDEX_NONE, NumSkeletonBones);

	int32 NumInertializationBones = 0;

	for (const FCompactPoseBoneIndex BoneIndex : Pose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex == INDEX_NONE)
		{
			continue;
		}

		// For each valid bone in the Compact Pose we write into BoneIndices the InertializationBoneIndex -
		// i.e. the index into BoneTranslations, BoneRotations, and BoneScales we are going to use to store
		// the transform data

		BoneIndices[SkeletonPoseBoneIndex] = NumInertializationBones;
		NumInertializationBones++;
	}

	// Initialize the BoneTranslations, BoneRotations, and BoneScales arrays

	BoneTranslations.Init(FVector::ZeroVector, NumInertializationBones);
	BoneRotations.Init(FQuat::Identity, NumInertializationBones);
	BoneScales.Init(FVector::OneVector, NumInertializationBones);

	for (const FCompactPoseBoneIndex BoneIndex : Pose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex == INDEX_NONE)
		{
			continue;
		}

		// Get the InertializationBoneIndex and write the transform data

		const int32 InertializationBoneIndex = BoneIndices[SkeletonPoseBoneIndex];
		check(InertializationBoneIndex != INDEX_NONE);

		const FTransform BoneTransform = Pose[BoneIndex];
		BoneTranslations[InertializationBoneIndex] = BoneTransform.GetTranslation();
		BoneRotations[InertializationBoneIndex] = BoneTransform.GetRotation();
		BoneScales[InertializationBoneIndex] = BoneTransform.GetScale3D();
	}

	// Init the Root Motion Delta

	bHasRootMotion = false;
	RootMotionDelta = FTransform::Identity;

	if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
	{
		bHasRootMotion = RootMotionProvider->ExtractRootMotion(Attributes, RootMotionDelta);
	}

	// Init the rest of the snapshot data

	Curves.InitFrom(InCurves);
	ComponentTransform = InComponentTransform;
	AttachParentName = InAttachParentName;
	DeltaTime = InDeltaTime;
}

bool FInertializationSparsePose::IsEmpty() const
{
	return BoneIndices.IsEmpty();
}

void FInertializationSparsePose::Empty()
{
	BoneIndices.Empty();
	BoneTranslations.Empty();
	BoneRotations.Empty();
	BoneScales.Empty();
	Curves.BlendedCurve.Empty();
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS

// DEPRECATED: See FAnimNode_Inertialization::InitFrom
void FInertializationPoseDiff::InitFrom(const FCompactPose& Pose, const FBlendedCurve& Curves, const FTransform& ComponentTransform, const FName& AttachParentName, const FInertializationPose& Prev1, const FInertializationPose& Prev2, const UE::Anim::FCurveFilter& CurveFilter)
{
	const FBoneContainer& BoneContainer = Pose.GetBoneContainer();

	const FQuat ComponentTransform_GetRotation_Inverse = ComponentTransform.GetRotation().Inverse();

	// Determine if we should initialize in local space (the default) or in world space (for situations where we wish to correct
	// a world-space discontinuity such as an abrupt orientation change)
	InertializationSpace = EInertializationSpace::Default;
	if (AttachParentName != Prev1.AttachParentName || AttachParentName != Prev2.AttachParentName)
	{
		// If the parent space has changed, then inertialize in world space
		InertializationSpace = EInertializationSpace::WorldSpace;
	}
	else if (AttachParentName == NAME_None)
	{
		// If there was a discontinuity in ComponentTransform orientation, then correct for that by inertializing the orientation in world space
		// (but only if the mesh is not attached to another actor, because we don't want to dampen the connection between attached actors)
		if ((FMath::Abs((Prev1.ComponentTransform.GetRotation() * ComponentTransform_GetRotation_Inverse).W) < 0.999f) ||	// (W < 0.999f --> angle > 5 degrees)
			(FMath::Abs((Prev2.ComponentTransform.GetRotation() * ComponentTransform_GetRotation_Inverse).W) < 0.999f))		// (W < 0.999f --> angle > 5 degrees)
		{
			InertializationSpace = EInertializationSpace::WorldRotation;
		}
	}

	// Compute the inertialization differences for each bone
	const int32 NumSkeletonBones = UE::Anim::Inertialization::Private::GetNumSkeletonBones(BoneContainer);
	BoneDiffs.Empty(NumSkeletonBones);
	BoneDiffs.AddZeroed(NumSkeletonBones);
	for (FCompactPoseBoneIndex BoneIndex : Pose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex != INDEX_NONE && Prev1.BoneStates[SkeletonPoseBoneIndex] == EInertializationBoneState::Valid)
		{
			const FTransform PoseTransform = Pose[BoneIndex];
			FTransform Prev1Transform = Prev1.BoneTransforms[SkeletonPoseBoneIndex];
			FTransform Prev2Transform = Prev2.BoneTransforms[SkeletonPoseBoneIndex];
			const bool Prev2IsValid = Prev2.BoneStates[SkeletonPoseBoneIndex] == EInertializationBoneState::Valid;

			if (BoneIndex.IsRootBone())
			{
				// If we are inertializing in world space, then adjust the historical root bones to be in a consistent reference frame
				if (InertializationSpace == EInertializationSpace::WorldSpace)
				{
					Prev1Transform *= Prev1.ComponentTransform.GetRelativeTransform(ComponentTransform);
					Prev2Transform *= Prev2.ComponentTransform.GetRelativeTransform(ComponentTransform);
				}
				else if (InertializationSpace == EInertializationSpace::WorldRotation)
				{
					Prev1Transform.SetRotation(ComponentTransform_GetRotation_Inverse * Prev1.ComponentTransform.GetRotation() * Prev1Transform.GetRotation());
					Prev2Transform.SetRotation(ComponentTransform_GetRotation_Inverse * Prev2.ComponentTransform.GetRotation() * Prev2Transform.GetRotation());
				}
			}
			else
			{
				// If this bone is a child of an excluded bone, then adjust the previous transforms to be relative to the excluded parent's
				// new transform so that the children maintain their original component space transform even though the parent will pop
				FCompactPoseBoneIndex ParentBoneIndex = BoneContainer.GetParentBoneIndex(BoneIndex);
				int32 ParentSkeletonPoseBoneIndex = (ParentBoneIndex != INDEX_NONE) ? BoneContainer.GetSkeletonIndex(ParentBoneIndex) : INDEX_NONE;
				if (ParentBoneIndex != INDEX_NONE && ParentSkeletonPoseBoneIndex != INDEX_NONE &&
					(Prev1.BoneStates[ParentSkeletonPoseBoneIndex] == EInertializationBoneState::Excluded || Prev2.BoneStates[ParentSkeletonPoseBoneIndex] == EInertializationBoneState::Excluded))
				{
					FTransform ParentPrev1Transform = Prev1.BoneTransforms[ParentSkeletonPoseBoneIndex];
					FTransform ParentPrev2Transform = Prev2.BoneTransforms[ParentSkeletonPoseBoneIndex];
					FTransform ParentPoseTransform = Pose[ParentBoneIndex];

					// Continue walking up the skeleton hierarchy in case the parent's parent etc is also excluded
					ParentBoneIndex = BoneContainer.GetParentBoneIndex(ParentBoneIndex);
					ParentSkeletonPoseBoneIndex = (ParentBoneIndex != INDEX_NONE) ? BoneContainer.GetSkeletonIndex(ParentBoneIndex) : INDEX_NONE;
					while (ParentBoneIndex != INDEX_NONE && ParentSkeletonPoseBoneIndex != INDEX_NONE &&
						(Prev1.BoneStates[ParentSkeletonPoseBoneIndex] == EInertializationBoneState::Excluded || Prev2.BoneStates[ParentSkeletonPoseBoneIndex] == EInertializationBoneState::Excluded))
					{
						ParentPrev1Transform *= Prev1.BoneTransforms[ParentSkeletonPoseBoneIndex];
						ParentPrev2Transform *= Prev2.BoneTransforms[ParentSkeletonPoseBoneIndex];
						ParentPoseTransform *= Pose[ParentBoneIndex];

						ParentBoneIndex = BoneContainer.GetParentBoneIndex(ParentBoneIndex);
						ParentSkeletonPoseBoneIndex = (ParentBoneIndex != INDEX_NONE) ? BoneContainer.GetSkeletonIndex(ParentBoneIndex) : INDEX_NONE;
					}

					// Adjust the transforms so that they behave as though the excluded parent has been in its new location all along
					Prev1Transform *= ParentPrev1Transform.GetRelativeTransform(ParentPoseTransform);
					Prev2Transform *= ParentPrev2Transform.GetRelativeTransform(ParentPoseTransform);
				}
			}

			FInertializationBoneDiff& BoneDiff = BoneDiffs[SkeletonPoseBoneIndex];

			// Compute the bone translation difference
			{
				FVector TranslationDirection = FVector::ZeroVector;
				float TranslationMagnitude = 0.0f;
				float TranslationSpeed = 0.0f;

				const FVector T = Prev1Transform.GetTranslation() - PoseTransform.GetTranslation();
				TranslationMagnitude = T.Size();
				if (TranslationMagnitude > UE_KINDA_SMALL_NUMBER)
				{
					TranslationDirection = T / TranslationMagnitude;
				}

				if (Prev2IsValid && Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER && TranslationMagnitude > UE_KINDA_SMALL_NUMBER)
				{
					const FVector PrevT = Prev2Transform.GetTranslation() - PoseTransform.GetTranslation();
					const float PrevMagnitude = FVector::DotProduct(PrevT, TranslationDirection);
					TranslationSpeed = (TranslationMagnitude - PrevMagnitude) / Prev1.DeltaTime;
				}

				BoneDiff.TranslationDirection = TranslationDirection;
				BoneDiff.TranslationMagnitude = TranslationMagnitude;
				BoneDiff.TranslationSpeed = TranslationSpeed;
			}

			// Compute the bone rotation difference
			{
				FVector RotationAxis = FVector::ZeroVector;
				float RotationAngle = 0.0f;
				float RotationSpeed = 0.0f;

				const FQuat Q = Prev1Transform.GetRotation() * PoseTransform.GetRotation().Inverse();
				Q.ToAxisAndAngle(RotationAxis, RotationAngle);
				RotationAngle = FMath::UnwindRadians(RotationAngle);
				if (RotationAngle < 0.0f)
				{
					RotationAxis = -RotationAxis;
					RotationAngle = -RotationAngle;
				}

				if (Prev2IsValid && Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER && RotationAngle > UE_KINDA_SMALL_NUMBER)
				{
					const FQuat PrevQ = Prev2Transform.GetRotation() * PoseTransform.GetRotation().Inverse();
					const float PrevAngle = PrevQ.GetTwistAngle(RotationAxis);
					RotationSpeed = FMath::UnwindRadians(RotationAngle - PrevAngle) / Prev1.DeltaTime;
				}

				BoneDiff.RotationAxis = RotationAxis;
				BoneDiff.RotationAngle = RotationAngle;
				BoneDiff.RotationSpeed = RotationSpeed;
			}

			// Compute the bone scale difference
			{
				FVector ScaleAxis = FVector::ZeroVector;
				float ScaleMagnitude = 0.0f;
				float ScaleSpeed = 0.0f;

				const FVector S = Prev1Transform.GetScale3D() - PoseTransform.GetScale3D();
				ScaleMagnitude = S.Size();
				if (ScaleMagnitude > UE_KINDA_SMALL_NUMBER)
				{
					ScaleAxis = S / ScaleMagnitude;
				}

				if (Prev2IsValid && Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER && ScaleMagnitude > UE_KINDA_SMALL_NUMBER)
				{
					const FVector PrevS = Prev2Transform.GetScale3D() - PoseTransform.GetScale3D();
					const float PrevMagnitude = FVector::DotProduct(PrevS, ScaleAxis);
					ScaleSpeed = (ScaleMagnitude - PrevMagnitude) / Prev1.DeltaTime;
				}

				BoneDiff.ScaleAxis = ScaleAxis;
				BoneDiff.ScaleMagnitude = ScaleMagnitude;
				BoneDiff.ScaleSpeed = ScaleSpeed;
			}
		}
	}

	// Compute the curve differences
	// First copy in current values
	CurveDiffs.CopyFrom(Curves);

	// Compute differences
	UE::Anim::FNamedValueArrayUtils::Union(CurveDiffs, Prev1.Curves.BlendedCurve,
		[](FInertializationCurveDiffElement& OutResultElement, const UE::Anim::FCurveElement& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			OutResultElement.Delta = InElement1.Value - OutResultElement.Value;
		});

	// Compute derivatives
	if(Prev1.DeltaTime > UE_KINDA_SMALL_NUMBER)
	{
		UE::Anim::FNamedValueArrayUtils::Union(CurveDiffs, Prev2.Curves.BlendedCurve,
			[DeltaTime = Prev1.DeltaTime](FInertializationCurveDiffElement& OutResultElement, const UE::Anim::FCurveElement& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				const float Prev1Weight = OutResultElement.Delta - OutResultElement.Value;
				const float Prev2Weight = InElement1.Value;
				OutResultElement.Derivative = (Prev1Weight - Prev2Weight) / DeltaTime;
			});
	}

	// Apply filtering to diffs to remove anything we dont want to inertialize
	if(CurveFilter.Num() > 0)
	{
		UE::Anim::FCurveUtils::Filter(CurveDiffs, CurveFilter);
	}
}

// DEPRECATED: See FAnimNode_Inertialization::ApplyTo
void FInertializationPoseDiff::ApplyTo(FCompactPose& Pose, FBlendedCurve& Curves, float InertializationElapsedTime, float InertializationDuration, TArrayView<const float> InertializationDurationPerBone) const
{
	const FBoneContainer& BoneContainer = Pose.GetBoneContainer();

	// Apply pose difference
	for (FCompactPoseBoneIndex BoneIndex : Pose.ForEachBoneIndex())
	{
		const int32 SkeletonPoseBoneIndex = BoneContainer.GetSkeletonIndex(BoneIndex);

		if (SkeletonPoseBoneIndex != INDEX_NONE)
		{
			const FInertializationBoneDiff& BoneDiff = BoneDiffs[SkeletonPoseBoneIndex];
			const float Duration = InertializationDurationPerBone[SkeletonPoseBoneIndex];

			// Apply the bone translation difference
			const FVector T = BoneDiff.TranslationDirection *
				UE::Anim::Inertialization::Private::CalcInertialFloat(BoneDiff.TranslationMagnitude, BoneDiff.TranslationSpeed, InertializationElapsedTime, Duration);
			Pose[BoneIndex].AddToTranslation(T);

			// Apply the bone rotation difference
			const FQuat Q = FQuat(BoneDiff.RotationAxis,
				UE::Anim::Inertialization::Private::CalcInertialFloat(BoneDiff.RotationAngle, BoneDiff.RotationSpeed, InertializationElapsedTime, Duration));
			Pose[BoneIndex].SetRotation(Q * Pose[BoneIndex].GetRotation());

			// Apply the bone scale difference
			const FVector S = BoneDiff.ScaleAxis *
				UE::Anim::Inertialization::Private::CalcInertialFloat(BoneDiff.ScaleMagnitude, BoneDiff.ScaleSpeed, InertializationElapsedTime, Duration);
			Pose[BoneIndex].SetScale3D(S + Pose[BoneIndex].GetScale3D());
		}
	}

	Pose.NormalizeRotations();

	// Apply curve differences
	UE::Anim::FNamedValueArrayUtils::Union(Curves, CurveDiffs,
		[&InertializationElapsedTime, &InertializationDuration](UE::Anim::FCurveElement& OutResultElement, const FInertializationCurveDiffElement& InParamElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			OutResultElement.Value += UE::Anim::Inertialization::Private::CalcInertialFloat(InParamElement.Delta, InParamElement.Derivative, InertializationElapsedTime, InertializationDuration);
			OutResultElement.Flags |= InParamElement.Flags;
		});
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS


#undef LOCTEXT_NAMESPACE
