// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_LinkedAnimGraph.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "Animation/AnimNode_Root.h"
#include "Animation/AnimTrace.h"
#include "Animation/BlendProfile.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/ExposedValueHandler.h"
#include "ObjectTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_LinkedAnimGraph)

FAnimNode_LinkedAnimGraph::FAnimNode_LinkedAnimGraph()
	: InstanceClass(nullptr)
#if WITH_EDITORONLY_DATA
	, Tag_DEPRECATED(NAME_None)
#endif
	, LinkedRoot(nullptr)
	, NodeIndex(INDEX_NONE)
	, CachedLinkedNodeIndex(INDEX_NONE)
	, PendingBlendOutDuration(-1.0f)
	, PendingBlendOutProfile(nullptr)
	, PendingBlendInDuration(-1.0f)
	, PendingBlendInProfile(nullptr)
	, bReceiveNotifiesFromLinkedInstances(false)
	, bPropagateNotifiesToLinkedInstances(false)
{
}

void FAnimNode_LinkedAnimGraph::InitializeSubGraph_AnyThread(const FAnimationInitializeContext& Context)
{
	UAnimInstance* InstanceToRun = GetTargetInstance<UAnimInstance>();
	if(InstanceToRun && LinkedRoot)
	{
		FAnimInstanceProxy& Proxy = InstanceToRun->GetProxyOnAnyThread<FAnimInstanceProxy>();

		// Make sure we have valid objects in place for the sub-graph init
		Proxy.InitializeObjects(InstanceToRun);

		Proxy.InitializationCounter.SynchronizeWith(Context.AnimInstanceProxy->InitializationCounter);
		Proxy.InitializeRootNode_WithRoot(LinkedRoot);
	}
}

void FAnimNode_LinkedAnimGraph::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_Base::Initialize_AnyThread(Context);

	InitializeSubGraph_AnyThread(Context);

	// Make sure we propagate down all input poses, as they may not all be linked in the linked graph
	for(FPoseLink& InputPose : InputPoses)
	{
		InputPose.Initialize(Context);
	}
}

void FAnimNode_LinkedAnimGraph::CacheBonesSubGraph_AnyThread(const FAnimationCacheBonesContext& Context)
{
	UAnimInstance* InstanceToRun = GetTargetInstance<UAnimInstance>();
	if(InstanceToRun && LinkedRoot)
	{
		FAnimInstanceProxy& Proxy = InstanceToRun->GetProxyOnAnyThread<FAnimInstanceProxy>();
		Proxy.CachedBonesCounter.SynchronizeWith(Context.AnimInstanceProxy->CachedBonesCounter);

		// Note not calling Proxy.CacheBones_WithRoot here as it is guarded by
		// bBoneCachesInvalidated, which is handled at a higher level
		FAnimationCacheBonesContext LinkedContext(&Proxy);
		LinkedContext.SetNodeId(CachedLinkedNodeIndex);
		LinkedRoot->CacheBones_AnyThread(LinkedContext);
	}
}

void FAnimNode_LinkedAnimGraph::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	CacheBonesSubGraph_AnyThread(Context);

	// Make sure we propagate down all input poses, as they may not all be linked in the linked graph
	for(FPoseLink& InputPose : InputPoses)
	{
		InputPose.CacheBones(Context);
	}
}

void FAnimNode_LinkedAnimGraph::Update_AnyThread(const FAnimationUpdateContext& InContext)
{
	GetEvaluateGraphExposedInputs().Execute(InContext);

	UAnimInstance* InstanceToRun = GetTargetInstance<UAnimInstance>();
	if(InstanceToRun && LinkedRoot)
	{
		FAnimInstanceProxy& Proxy = InstanceToRun->GetProxyOnAnyThread<FAnimInstanceProxy>();
		Proxy.UpdateCounter.SynchronizeWith(InContext.AnimInstanceProxy->UpdateCounter);

		PropagateInputProperties(InContext.AnimInstanceProxy->GetAnimInstanceObject());

		// We can call this unconditionally here now because linked anim instances are forced to have a parallel update
		// in USkeletalMeshComponent::TickAnimation. It used to be the case that we could do non-parallel work in 
		// USkeletalMeshComponent::TickAnimation, which would mean we would have to skip doing that work here.
		FAnimationUpdateContext NewContext = InContext.WithOtherProxy(&Proxy);
		NewContext.SetNodeId(INDEX_NONE);
		NewContext.SetNodeId(CachedLinkedNodeIndex);
		Proxy.UpdateAnimation_WithRoot(NewContext, LinkedRoot, GetDynamicLinkFunctionName());
	}
	else if(InputPoses.Num() > 0)
	{
		// If we have no valid instance (self or otherwise), we need to propagate down the graph to make sure
		// subsequent nodes get properly updated
		InputPoses[0].Update(InContext);
	}

	// Consume pending inertial blend requests
	if(PendingBlendOutDuration >= 0.0f || PendingBlendInDuration >= 0.0f)
	{
		UE::Anim::IInertializationRequester* InertializationRequester = InContext.GetMessage<UE::Anim::IInertializationRequester>();
		if(InertializationRequester)
		{
			// Issue the pending inertialization requests (which will get merged together by the inertialization node itself)
			if (PendingBlendOutDuration >= 0.0f)
			{
				FInertializationRequest Request;
				Request.Duration = PendingBlendOutDuration;
				Request.BlendProfile = PendingBlendOutProfile;
#if ANIM_TRACE_ENABLED
				Request.DescriptionString = NSLOCTEXT("AnimNode_LinkedAnimGraph", "InertializationRequestDescriptionOut", "Out").ToString();
				Request.NodeId = InContext.GetCurrentNodeId();
				Request.AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
#endif

				InertializationRequester->RequestInertialization(Request);
			}

			if (PendingBlendInDuration >= 0.0f)
			{
				FInertializationRequest Request;
				Request.Duration = PendingBlendInDuration;
				Request.BlendProfile = PendingBlendInProfile;
#if ANIM_TRACE_ENABLED
				Request.DescriptionString = NSLOCTEXT("AnimNode_LinkedAnimGraph", "InertializationRequestDescriptionIn", "In").ToString();
				Request.NodeId = InContext.GetCurrentNodeId();
				Request.AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
#endif

				InertializationRequester->RequestInertialization(Request);
			}

			InertializationRequester->AddDebugRecord(*InContext.AnimInstanceProxy, InContext.GetCurrentNodeId());
		}
		else if ((PendingBlendOutDuration != 0.0f) && (PendingBlendInDuration != 0.0f) && (InputPoses.Num() > 0))
		{
			FAnimNode_Inertialization::LogRequestError(InContext, InputPoses[0]);
		}
	}
	PendingBlendOutDuration = -1.0f;
	PendingBlendOutProfile = nullptr;
	PendingBlendInDuration = -1.0f;
	PendingBlendInProfile = nullptr;

	TRACE_ANIM_NODE_VALUE(InContext, TEXT("Name"), GetDynamicLinkFunctionName());
	TRACE_ANIM_NODE_VALUE(InContext, TEXT("Target Class"), InstanceClass.Get());
}

void FAnimNode_LinkedAnimGraph::Evaluate_AnyThread(FPoseContext& Output)
{
#if	ANIMNODE_STATS_VERBOSE
	// Record name of linked graph we are updating
	FScopeCycleCounter LinkedAnimGraphNameCycleCounter(StatID);
#endif // ANIMNODE_STATS_VERBOSE

	UAnimInstance* InstanceToRun = GetTargetInstance<UAnimInstance>();
	if(InstanceToRun && LinkedRoot)
	{
		// Stash current proxy for restoration after recursion
		FAnimInstanceProxy& OldProxy = *Output.AnimInstanceProxy;

		FAnimInstanceProxy& Proxy = InstanceToRun->GetProxyOnAnyThread<FAnimInstanceProxy>();
		Proxy.EvaluationCounter.SynchronizeWith(Output.AnimInstanceProxy->EvaluationCounter);
		Output.Pose.SetBoneContainer(&Proxy.GetRequiredBones());
		Output.AnimInstanceProxy = &Proxy;
		Output.SetNodeId(INDEX_NONE);
		Output.SetNodeId(CachedLinkedNodeIndex);

		// Run the anim blueprint
		Proxy.EvaluateAnimation_WithRoot(Output, LinkedRoot);

		// Restore proxy & required bones after evaluation
		Output.AnimInstanceProxy = &OldProxy;
		Output.Pose.SetBoneContainer(&OldProxy.GetRequiredBones());
	}
	else if(InputPoses.Num() > 0)
	{
		// If we have no valid instance (self or otherwise), we need to propagate down the graph to make sure
		// subsequent nodes get properly evaluated
		InputPoses[0].Evaluate(Output);
	}
	else
	{
		Output.ResetToRefPose();
	}
}

void FAnimNode_LinkedAnimGraph::GatherDebugData(FNodeDebugData& DebugData)
{
	// Add our entry
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("Target: %s"), (*InstanceClass) ? *InstanceClass->GetName() : TEXT("None"));

	DebugData.AddDebugItem(DebugLine);

	UAnimInstance* InstanceToRun = GetTargetInstance<UAnimInstance>();
	// Gather data from the linked instance
	if(InstanceToRun && LinkedRoot)
	{
		FAnimInstanceProxy& Proxy = InstanceToRun->GetProxyOnAnyThread<FAnimInstanceProxy>();
		Proxy.GatherDebugData_WithRoot(DebugData.BranchFlow(1.0f), LinkedRoot, GetDynamicLinkFunctionName());
	}
	else if(InputPoses.Num() > 0)
	{
		// If we have no valid instance (self or otherwise), we need to propagate down the graph to make sure
		// subsequent nodes get their debug data properly collected to reflect relevancy
		InputPoses[0].GatherDebugData(DebugData);
	}
}

void FAnimNode_LinkedAnimGraph::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
#if WITH_EDITORONLY_DATA
	SourceInstance = const_cast<UAnimInstance*>(InAnimInstance);
#endif
	
	UAnimInstance* InstanceToRun = GetTargetInstance<UAnimInstance>();

	if(*InstanceClass)
	{
		ReinitializeLinkedAnimInstance(InAnimInstance);
	}
	else if(InstanceToRun)
	{
		// We have an instance but no instance class
		TeardownInstance(InAnimInstance);
	}

	if(InstanceToRun)
	{
		InstanceToRun->GetProxyOnAnyThread<FAnimInstanceProxy>().InitializeObjects(InstanceToRun);
	}
}

void FAnimNode_LinkedAnimGraph::TeardownInstance(const UAnimInstance* InOwningAnimInstance)
{
	UAnimInstance* InstanceToRun = GetTargetInstance<UAnimInstance>();
	if (InstanceToRun)
	{
		// trace lifetime end early, because by the time we get UninitializeAnimation below, the Owner has changed, and so the ObjectId has changed.
		DynamicUnlink(const_cast<UAnimInstance*>(InOwningAnimInstance));
		// Never delete the owning animation instance
		if (InstanceToRun != InOwningAnimInstance)
		{
			if (CanTeardownLinkedInstance(InstanceToRun))
			{
				TRACE_OBJECT_LIFETIME_END(InstanceToRun);
				USkeletalMeshComponent* MeshComp = InOwningAnimInstance->GetSkelMeshComponent();
				check(MeshComp);
				MeshComp->GetLinkedAnimInstances().Remove(InstanceToRun);
				// Only call UninitializeAnimation if we are not the owning anim instance
				InstanceToRun->UninitializeAnimation();
				InstanceToRun->MarkAsGarbage();
			}
		}

		InstanceToRun = nullptr;
	}

	SetTargetInstance(nullptr);
}

void FAnimNode_LinkedAnimGraph::ReinitializeLinkedAnimInstance(const UAnimInstance* InOwningAnimInstance, UAnimInstance* InNewAnimInstance)
{
	UAnimInstance* InstanceToRun = GetTargetInstance<UAnimInstance>();

	IAnimClassInterface* PriorAnimBPClass = InstanceToRun ? IAnimClassInterface::GetFromClass(InstanceToRun->GetClass()) : nullptr;

	// Full reinit, kill old instances
	TeardownInstance(InOwningAnimInstance);

	if(*InstanceClass || InNewAnimInstance)
	{
		USkeletalMeshComponent* MeshComp = InOwningAnimInstance->GetSkelMeshComponent();
		check(MeshComp);

		// Need an instance to run, so create it now
		InstanceToRun = InNewAnimInstance ? InNewAnimInstance : NewObject<UAnimInstance>(MeshComp, InstanceClass);
		if (InNewAnimInstance == nullptr)
		{
			// if incoming AnimInstance was null, it was created by this function
			// we mark them as created by linked anim graph
			// this is to know who owns memory instance
			InstanceToRun->bCreatedByLinkedAnimGraph = true;
			InstanceToRun->bPropagateNotifiesToLinkedInstances = bPropagateNotifiesToLinkedInstances;
			InstanceToRun->bReceiveNotifiesFromLinkedInstances = bReceiveNotifiesFromLinkedInstances;
		}

		SetTargetInstance(InstanceToRun);

		// Link before we call InitializeAnimation() so we propgate the call to linked input poses
		DynamicLink(const_cast<UAnimInstance*>(InOwningAnimInstance));

		if(InNewAnimInstance == nullptr)
		{
			// Initialize the new instance
			InstanceToRun->InitializeAnimation();

			if(MeshComp->HasBegunPlay())
			{
				InstanceToRun->NativeBeginPlay();
				InstanceToRun->BlueprintBeginPlay();
			}

			MeshComp->GetLinkedAnimInstances().Add(InstanceToRun);
		}

		InitializeProperties(InOwningAnimInstance, InstanceToRun->GetClass());

		IAnimClassInterface* NewAnimBPClass = IAnimClassInterface::GetFromClass(InstanceToRun->GetClass());

		RequestBlend(PriorAnimBPClass, NewAnimBPClass);
	}
	else
	{
		RequestBlend(PriorAnimBPClass, nullptr);
	}
}

void FAnimNode_LinkedAnimGraph::SetAnimClass(TSubclassOf<UAnimInstance> InClass, const UAnimInstance* InOwningAnimInstance)
{
	UClass* NewClass = InClass.Get();

	// Make sure this is a valid blueprint class
	if (NewClass && IAnimClassInterface::GetFromClass(NewClass) == nullptr)
	{
		ensure(false);
		return;
	}

	// Verified OK, so set it now
	TSubclassOf<UAnimInstance> OldClass = InstanceClass;
	InstanceClass = InClass;

	if(InstanceClass != OldClass)
	{
		ReinitializeLinkedAnimInstance(InOwningAnimInstance);
	}
}

FName FAnimNode_LinkedAnimGraph::GetDynamicLinkFunctionName() const
{
	return NAME_AnimGraph;
}

UAnimInstance* FAnimNode_LinkedAnimGraph::GetDynamicLinkTarget(UAnimInstance* InOwningAnimInstance) const
{
	return GetTargetInstance<UAnimInstance>();
}

void FAnimNode_LinkedAnimGraph::DynamicLink(UAnimInstance* InOwningAnimInstance)
{
	CachedLinkedNodeIndex = INDEX_NONE;

	UAnimInstance* LinkTargetInstance = GetDynamicLinkTarget(InOwningAnimInstance);
	if(LinkTargetInstance)
	{
		IAnimClassInterface* SubAnimBlueprintClass = IAnimClassInterface::GetFromClass(LinkTargetInstance->GetClass());
		if(SubAnimBlueprintClass)
		{
			FAnimInstanceProxy* NonConstProxy = &InOwningAnimInstance->GetProxyOnAnyThread<FAnimInstanceProxy>();
			const FName FunctionToLink = GetDynamicLinkFunctionName();

			// Link input poses
			const TArray<FAnimBlueprintFunction>& AnimBlueprintFunctions = SubAnimBlueprintClass->GetAnimBlueprintFunctions();
			const int32 FunctionCount = AnimBlueprintFunctions.Num();
			for(int32 FunctionIndex = 0; FunctionIndex < FunctionCount; ++FunctionIndex)
			{
				const FAnimBlueprintFunction& AnimBlueprintFunction = AnimBlueprintFunctions[FunctionIndex];

				if(AnimBlueprintFunction.Name == FunctionToLink)
				{
					CachedLinkedNodeIndex = AnimBlueprintFunction.OutputPoseNodeIndex;

					for(int32 InputPoseIndex = 0; InputPoseIndex < AnimBlueprintFunction.InputPoseNames.Num() && InputPoseIndex < InputPoses.Num(); ++InputPoseIndex)
					{
						// Make sure we attempt a re-link first, as only this pose link knows its target
						FAnimationInitializeContext Context(NonConstProxy);
						InputPoses[InputPoseIndex].AttemptRelink(Context);

						int32 InputPropertyIndex = FindFunctionInputIndex(AnimBlueprintFunction, AnimBlueprintFunction.InputPoseNames[InputPoseIndex]);
						if(InputPropertyIndex != INDEX_NONE && AnimBlueprintFunction.InputPoseNodeProperties[InputPropertyIndex])
						{
							FAnimNode_LinkedInputPose* LinkedInputPoseNode = AnimBlueprintFunction.InputPoseNodeProperties[InputPropertyIndex]->ContainerPtrToValuePtr<FAnimNode_LinkedInputPose>(LinkTargetInstance);
							check(LinkedInputPoseNode->Name == AnimBlueprintFunction.InputPoseNames[InputPoseIndex]);
							LinkedInputPoseNode->DynamicLink(NonConstProxy, &InputPoses[InputPoseIndex], NodeIndex);
						}
						else
						{
							UE_LOGF(LogAnimation, Warning, "Unable to dynamically link input pose %ls.", *AnimBlueprintFunction.InputPoseNames[InputPoseIndex].ToString());
						}
					}

					if(AnimBlueprintFunction.OutputPoseNodeProperty)
					{
						LinkedRoot = AnimBlueprintFunction.OutputPoseNodeProperty->ContainerPtrToValuePtr<FAnimNode_Root>(LinkTargetInstance);
					}
					else
					{
						UE_LOGF(LogAnimation, Warning, "Unable to dynamically link root %ls.", *FunctionToLink.ToString());

						// Clear our linked root so that future InitializeSubGraph_AnyThread / etc calls will no-op, clear index as well
						LinkedRoot = nullptr;
						CachedLinkedNodeIndex = INDEX_NONE;
					}

					break;
				}
			}
		}
	}
}

void FAnimNode_LinkedAnimGraph::DynamicUnlink(UAnimInstance* InOwningAnimInstance)
{
	// unlink root
	LinkedRoot = nullptr;

	// unlink input poses
	UAnimInstance* LinkTargetInstance = GetDynamicLinkTarget(InOwningAnimInstance);
	if(LinkTargetInstance)
	{
		IAnimClassInterface* SubAnimBlueprintClass = IAnimClassInterface::GetFromClass(LinkTargetInstance->GetClass());
		if(SubAnimBlueprintClass)
		{
			const FName FunctionToLink = GetDynamicLinkFunctionName();

			// Link input poses
			for(const FAnimBlueprintFunction& AnimBlueprintFunction : SubAnimBlueprintClass->GetAnimBlueprintFunctions())
			{
				if(AnimBlueprintFunction.Name == FunctionToLink)
				{
					for(int32 InputPoseIndex = 0; InputPoseIndex < AnimBlueprintFunction.InputPoseNames.Num() && InputPoseIndex < InputPoses.Num(); ++InputPoseIndex)
					{
						int32 InputPropertyIndex = FindFunctionInputIndex(AnimBlueprintFunction, AnimBlueprintFunction.InputPoseNames[InputPoseIndex]);
						if(InputPropertyIndex != INDEX_NONE && AnimBlueprintFunction.InputPoseNodeProperties[InputPropertyIndex])
						{
							FAnimNode_LinkedInputPose* LinkedInputPoseNode = AnimBlueprintFunction.InputPoseNodeProperties[InputPropertyIndex]->ContainerPtrToValuePtr<FAnimNode_LinkedInputPose>(LinkTargetInstance);
							check(LinkedInputPoseNode->Name == AnimBlueprintFunction.InputPoseNames[InputPoseIndex]);
							LinkedInputPoseNode->DynamicUnlink();
						}
						else
						{
							UE_LOGF(LogAnimation, Warning, "Unable to dynamically unlink input pose %ls.", *AnimBlueprintFunction.InputPoseNames[InputPoseIndex].ToString());
						}
					}

					break;
				}
			}
		}
	}

	CachedLinkedNodeIndex = INDEX_NONE;
}

int32 FAnimNode_LinkedAnimGraph::FindFunctionInputIndex(const FAnimBlueprintFunction& InAnimBlueprintFunction, const FName& InInputName)
{
	for(int32 InputPropertyIndex = 0; InputPropertyIndex < InAnimBlueprintFunction.InputPoseNames.Num(); ++InputPropertyIndex)
	{
		if(InInputName == InAnimBlueprintFunction.InputPoseNames[InputPropertyIndex])
		{
			return InputPropertyIndex;
		}
	}

	return INDEX_NONE;
}

void FAnimNode_LinkedAnimGraph::RequestBlend(const IAnimClassInterface* PriorAnimBPClass, const IAnimClassInterface* NewAnimBPClass)
{
	const FName Layer = GetDynamicLinkFunctionName();

	const FAnimGraphBlendOptions* PriorBlendOptions = PriorAnimBPClass ? PriorAnimBPClass->GetGraphBlendOptions().Find(Layer) : nullptr;
	const FAnimGraphBlendOptions* NewBlendOptions = NewAnimBPClass ? NewAnimBPClass->GetGraphBlendOptions().Find(Layer) : nullptr;

	if (PriorBlendOptions && PriorBlendOptions->BlendOutTime >= 0.0f)
	{
		PendingBlendOutDuration = PriorBlendOptions->BlendOutTime;
		PendingBlendOutProfile = PriorBlendOptions->BlendOutProfile;
	}
	else
	{
		PendingBlendOutDuration = -1.0f;
		PendingBlendOutProfile = nullptr;
	}

	if (NewBlendOptions && NewBlendOptions->BlendInTime >= 0.0f)
	{
		PendingBlendInDuration = NewBlendOptions->BlendInTime;
		PendingBlendInProfile = NewBlendOptions->BlendInProfile;
	}
	else
	{
		PendingBlendInDuration = -1.0f;
		PendingBlendInProfile = nullptr;
	}
}

#if WITH_EDITOR
void FAnimNode_LinkedAnimGraph::HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	static IConsoleVariable* UseLegacyAnimInstanceReinstancingBehavior = IConsoleManager::Get().FindConsoleVariable(TEXT("bp.UseLegacyAnimInstanceReinstancingBehavior"));
	if(UseLegacyAnimInstanceReinstancingBehavior == nullptr || !UseLegacyAnimInstanceReinstancingBehavior->GetBool())
	{
		UAnimInstance* SourceAnimInstance = CastChecked<UAnimInstance>(InSourceObject);

		FAnimInstanceProxy& SourceProxy = SourceAnimInstance->GetProxyOnAnyThread<FAnimInstanceProxy>();

		// Call Initialize here to ensure any custom proxies are initialized (as they may have been re-created during
		// re-instancing, and they dont call the constructor that takes a UAnimInstance*)
		SourceProxy.Initialize(SourceAnimInstance);

		// Similarly call Initialize here to catch any custom target proxies
		UAnimInstance* TargetAnimInstance = CastChecked<UAnimInstance>(InTargetObject);
		FAnimInstanceProxy& TargetProxy = TargetAnimInstance->GetProxyOnAnyThread<FAnimInstanceProxy>();
		TargetProxy.Initialize(TargetAnimInstance);

		// Reinstancing copies properties using UEngine::CopyPropertiesForUnrelatedObjects which is named-based, but a graph compilation can shuffle the node names around, so at this point
		// TargetAnimInstance can differ from InstanceClass. If we call InitializeProperties in this scenario, we read property offsets from the wrong class and this results in memory corruption,
		// when calling PropagateInputProperties, so we skip it. This does not fix the problem but it prevents the crash.
		UClass* TargetClass = GetTargetClass();
		if(TargetClass && InTargetObject->IsA(TargetClass))
		{
			InitializeProperties(SourceAnimInstance, TargetClass);
		}
		DynamicUnlink(SourceAnimInstance);
		DynamicLink(SourceAnimInstance);

		SourceProxy.InitializeCachedClassData();

		// Ensure we have a valid mesh at this point, as calling into the graph without one can result in crashes
		// as we assume a valid bone container/reference skeleton is present
		USkeletalMeshComponent* MeshComponent = SourceAnimInstance->GetSkelMeshComponent();
		if(MeshComponent && MeshComponent->IsRegistered() && MeshComponent->GetSkeletalMeshAsset())
		{
			SourceAnimInstance->RecalcRequiredBones();

			FAnimationInitializeContext Context(&SourceProxy);
			InitializeSubGraph_AnyThread(Context);
		}
	}
}
#endif

#if ANIMNODE_STATS_VERBOSE
void FAnimNode_LinkedAnimGraph::InitializeStatID()
{
	if (GetTargetInstance<UAnimInstance>())
	{
		StatID = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_Anim>(GetTargetInstance<UAnimInstance>()->GetClass()->GetName());
	}
	else
	{
		Super::InitializeStatID();
	}
}
#endif // ANIMNODE_STATS_VERBOSE
