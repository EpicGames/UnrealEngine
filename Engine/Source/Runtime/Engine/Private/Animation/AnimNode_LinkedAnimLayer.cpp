// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_LinkedAnimLayer.h"
#include "Animation/AnimSubsystem_SharedLinkedAnimLayers.h"
#include "Animation/AnimInstanceProxy.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_LinkedAnimLayer)

FName FAnimNode_LinkedAnimLayer::GetDynamicLinkFunctionName() const
{
	return Layer;
}

UAnimInstance* FAnimNode_LinkedAnimLayer::GetDynamicLinkTarget(UAnimInstance* InOwningAnimInstance) const
{
	if(Interface.Get())
	{
		return GetTargetInstance<UAnimInstance>();
	}
	else
	{
		return InOwningAnimInstance;
	}
}

void FAnimNode_LinkedAnimLayer::InitializeSourceProperties(const UAnimInstance* InAnimInstance)
{
	const int32 NumSourceProperties = SourcePropertyNames.Num();
	SourceProperties.SetNumZeroed(NumSourceProperties);

	const UClass* ThisClass = InAnimInstance->GetClass();
	for(int32 SourcePropertyIndex = 0; SourcePropertyIndex < NumSourceProperties; ++SourcePropertyIndex)
	{
		SourceProperties[SourcePropertyIndex] = ThisClass->FindPropertyByName(SourcePropertyNames[SourcePropertyIndex]);
	}
}

void FAnimNode_LinkedAnimLayer::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
#if WITH_EDITORONLY_DATA
	SourceInstance = const_cast<UAnimInstance*>(InAnimInstance);
#endif
	
	check(SourcePropertyNames.Num() == DestPropertyNames.Num());
	
	// Initialize source properties here as they do not change
	InitializeSourceProperties(InAnimInstance);

	// We only initialize here if we are running a 'self' layer. Layers that use external instances need to be 
	// initialized by the owning anim instance as they may share linked instances via grouping.
	if(Interface.Get() == nullptr || InstanceClass.Get() == nullptr)
	{
		InitializeSelfLayer(InAnimInstance);
	}
}

void FAnimNode_LinkedAnimLayer::OnUninitializeAnimInstance(UAnimInstance* InOwningAnimInstance)
{
	// Owning anim instance is being destroyed, clean dynamic layer data 
	if (UAnimInstance* CurrentTarget = GetTargetInstance<UAnimInstance>())
	{
		// Skip self layers
		if (CurrentTarget != InOwningAnimInstance)
		{
			USkeletalMeshComponent* MeshComp = InOwningAnimInstance->GetSkelMeshComponent();
			if (FAnimSubsystem_SharedLinkedAnimLayers* SharedLinkedAnimLayers = FAnimSubsystem_SharedLinkedAnimLayers::GetFromMesh(MeshComp))
			{
				// If target instance is a shared instance, unlink it when owner uninitialize
				if (SharedLinkedAnimLayers->IsSharedInstance(CurrentTarget))
				{
					DynamicUnlink(InOwningAnimInstance);
					SetTargetInstance(nullptr);
					CleanupSharedLinkedLayersData(InOwningAnimInstance, CurrentTarget);
				}
			}
		}
	}
}

void FAnimNode_LinkedAnimLayer::InitializeSelfLayer(const UAnimInstance* SelfAnimInstance)
{
	UAnimInstance* CurrentTarget = GetTargetInstance<UAnimInstance>();

	IAnimClassInterface* PriorAnimBPClass = CurrentTarget ? IAnimClassInterface::GetFromClass(CurrentTarget->GetClass()) : nullptr;

	USkeletalMeshComponent* MeshComp = SelfAnimInstance->GetSkelMeshComponent();
	check(MeshComp);

	if (LinkedRoot)
	{
		DynamicUnlink(const_cast<UAnimInstance*>(SelfAnimInstance));
	}

	// Switch from dynamic external to internal, kill old instance
	if (CurrentTarget && CurrentTarget != SelfAnimInstance)
	{
		if (CanTeardownLinkedInstance(CurrentTarget))
		{
			CurrentTarget->UninitializeAnimation();
			MeshComp->GetLinkedAnimInstances().Remove(CurrentTarget);
			CurrentTarget->MarkAsGarbage();
			CurrentTarget = nullptr;
		}
	}

	SetTargetInstance(const_cast<UAnimInstance*>(SelfAnimInstance));

	// Link before we call InitializeAnimation() so we propgate the call to linked input poses
	DynamicLink(const_cast<UAnimInstance*>(SelfAnimInstance));

	UClass* SelfClass = SelfAnimInstance->GetClass();
	InitializeProperties(SelfAnimInstance, SelfClass);

	IAnimClassInterface* NewAnimBPClass = IAnimClassInterface::GetFromClass(SelfClass);

	// No need for blending if the instance hasn't changed (this was causing issues when a layer was unlinked more than once in a single frame due to faulty blueprints, causing the good blend values to be stomped before being processed)
	if (CurrentTarget != GetTargetInstance<UAnimInstance>())
	{
		RequestBlend(PriorAnimBPClass, NewAnimBPClass);
	}
}

void FAnimNode_LinkedAnimLayer::SetLinkedLayerInstance(const UAnimInstance* InOwningAnimInstance, UAnimInstance* InNewLinkedInstance)
{
	UAnimInstance* PreviousTargetInstance = GetTargetInstance<UAnimInstance>();

	// Reseting to running as a self-layer, in case it is applicable
	if ((Interface.Get() == nullptr || InstanceClass.Get() == nullptr) && (InNewLinkedInstance == nullptr))
	{
		InitializeSelfLayer(InOwningAnimInstance);
	}
	else
	{
		ReinitializeLinkedAnimInstance(InOwningAnimInstance, InNewLinkedInstance);
	}

	if (PreviousTargetInstance != GetTargetInstance<UAnimInstance>())
	{
		CleanupSharedLinkedLayersData(InOwningAnimInstance, PreviousTargetInstance);
	}

#if WITH_EDITOR
	OnInstanceChangedEvent.Broadcast();
#endif
}

void FAnimNode_LinkedAnimLayer::InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass)
{
	check(SourcePropertyNames.Num() == DestPropertyNames.Num());

#if WITH_EDITOR
	// When reinstancing we need to init source properties here too as the class may have changed
	if(GIsReinstancing)
	{
		InitializeSourceProperties(CastChecked<UAnimInstance>(InSourceInstance));
	}
#endif
	
	// Build dest property list - source is set up when we initialize
	DestProperties.SetNumZeroed(SourcePropertyNames.Num());
	
	IAnimClassInterface* TargetAnimClassInterface = IAnimClassInterface::GetFromClass(InTargetClass);
	check(TargetAnimClassInterface);

	const FName FunctionName = GetDynamicLinkFunctionName();
	if(const FAnimBlueprintFunction* Function = IAnimClassInterface::FindAnimBlueprintFunction(TargetAnimClassInterface, FunctionName))
	{
		// Target properties are linked via the anim BP function params
		for(int32 DestPropertyIndex = 0; DestPropertyIndex < DestPropertyNames.Num(); ++DestPropertyIndex)
		{
			const FName& DestName = DestPropertyNames[DestPropertyIndex];

			// Look for an input property (parameter) with the specified name
			const int32 NumParams = Function->InputPropertyData.Num();
			for(int32 InputPropertyIndex = 0; InputPropertyIndex < NumParams; ++InputPropertyIndex)
			{
				if(Function->InputPropertyData[InputPropertyIndex].Name == DestName)
				{
					DestProperties[DestPropertyIndex] = Function->InputPropertyData[InputPropertyIndex].ClassProperty;
					break;
				}
			}
		}
	}
}

bool FAnimNode_LinkedAnimLayer::CanTeardownLinkedInstance(const UAnimInstance* LinkedInstance) const
{
	// Don't teardown instance that still have function linked to active shared instances
	USkeletalMeshComponent* MeshComp = LinkedInstance->GetSkelMeshComponent();
	if (FAnimSubsystem_SharedLinkedAnimLayers* SharedLinkedAnimLayers = FAnimSubsystem_SharedLinkedAnimLayers::GetFromMesh(MeshComp))
	{
		return !SharedLinkedAnimLayers->IsSharedInstance(LinkedInstance);
	}
	return true; 
}

void FAnimNode_LinkedAnimLayer::CleanupSharedLinkedLayersData(const UAnimInstance* InOwningAnimInstance, UAnimInstance* InPreviousTargetInstance)
{
	if (InPreviousTargetInstance)
	{
		USkeletalMeshComponent* MeshComp = InPreviousTargetInstance->GetSkelMeshComponent();
		if (FAnimSubsystem_SharedLinkedAnimLayers* SharedLinkedAnimLayers = FAnimSubsystem_SharedLinkedAnimLayers::GetFromMesh(MeshComp))
		{
			if (SharedLinkedAnimLayers->IsSharedInstance(InPreviousTargetInstance))
			{
				SharedLinkedAnimLayers->RemoveLinkedFunction(InPreviousTargetInstance, GetDynamicLinkFunctionName());
			}
		}
	}
}
#if WITH_EDITOR

void FAnimNode_LinkedAnimLayer::HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	static IConsoleVariable* UseLegacyAnimInstanceReinstancingBehavior = IConsoleManager::Get().FindConsoleVariable(TEXT("bp.UseLegacyAnimInstanceReinstancingBehavior"));
	if(UseLegacyAnimInstanceReinstancingBehavior == nullptr || !UseLegacyAnimInstanceReinstancingBehavior->GetBool())
	{
		UAnimInstance* SourceAnimInstance = CastChecked<UAnimInstance>(InSourceObject);

		// Skip if the owning mesh component is not registered. During BP reinstancing,
		// linked anim layer instances can survive on trashed/orphaned components (the old
		// pawn was destroyed but the linked AnimInstance is still subscribed to
		// OnObjectsReinstanced). The component's bone data was cleared in OnUnregister
		// but the mesh reference remains, so the existing mesh-only check is insufficient.
		USkeletalMeshComponent* MeshComponent = SourceAnimInstance->GetSkelMeshComponent();
		if (!MeshComponent || !MeshComponent->IsRegistered())
		{
			return;
		}

		FAnimInstanceProxy& SourceProxy = SourceAnimInstance->GetProxyOnAnyThread<FAnimInstanceProxy>();

		// Call Initialize here to ensure any custom proxies are initialized (as they may have been re-created during
		// re-instancing, and they dont call the constructor that takes a UAnimInstance*)
		SourceProxy.Initialize(SourceAnimInstance);

		InitializeProperties(SourceAnimInstance, Interface.Get() != nullptr ? Interface.Get() : InSourceObject->GetClass());
		DynamicUnlink(SourceAnimInstance);
		DynamicLink(SourceAnimInstance);

		SourceProxy.InitializeCachedClassData();

		if(MeshComponent->GetSkeletalMeshAsset())
		{
			SourceAnimInstance->RecalcRequiredBones();

			FAnimationInitializeContext Context(&SourceProxy);
			InitializeSubGraph_AnyThread(Context);
		}
	}
}

#endif


#if ANIMNODE_STATS_VERBOSE
void FAnimNode_LinkedAnimLayer::InitializeStatID()
{
	if (GetTargetInstance<UAnimInstance>())
	{
		FString StatName = GetTargetInstance<UAnimInstance>()->GetClass()->GetName() + " " + GetDynamicLinkFunctionName().ToString();
		StatID = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_Anim>(StatName);
	}
	else
	{
		Super::InitializeStatID();
	}
}
#endif // ANIMNODE_STATS_VERBOSE
