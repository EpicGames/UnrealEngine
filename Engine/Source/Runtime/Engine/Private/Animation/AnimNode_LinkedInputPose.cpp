// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_LinkedInputPose.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_LinkedInputPose)

const FName FAnimNode_LinkedInputPose::DefaultInputPoseName("InPose");

// Note not calling through Initialize or CacheBones here.
// This is handled in the owning LinkedAnimGraph node. This is because not all input poses may be linked in a
// particular linked graph, so to avoid mismatches in initialization and bone references we make sure that all
// branches of the tree are taken when initializing and caching bones.

#if ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG
void FAnimNode_LinkedInputPose::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	// Make sure to sync input pose debug counters as we still use this pose link in Update/Evaluate etc.
	InputPose.InitializationCounter.SynchronizeWith(Context.AnimInstanceProxy->GetInitializationCounter());
}

void FAnimNode_LinkedInputPose::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	// Make sure to sync input pose debug counters as we still use this pose link in Update/Evaluate etc.
	InputPose.CachedBonesCounter.SynchronizeWith(Context.AnimInstanceProxy->GetCachedBonesCounter());
}
#endif

void FAnimNode_LinkedInputPose::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	if(InputProxy)
	{
		FAnimationUpdateContext InputContext = Context.WithOtherProxy(InputProxy);
		InputContext.SetNodeId(OuterGraphNodeIndex);
		InputPose.Update(InputContext);
	}
}

void FAnimNode_LinkedInputPose::Evaluate_AnyThread(FPoseContext& Output)
{
	if(InputProxy)
	{
		// Stash current proxy for restoration after recursion
		FAnimInstanceProxy& OldProxy = *Output.AnimInstanceProxy;

		Output.AnimInstanceProxy = InputProxy;
		Output.Pose.SetBoneContainer(&InputProxy->GetRequiredBones());
		Output.SetNodeId(INDEX_NONE);
		Output.SetNodeId(OuterGraphNodeIndex);
		InputPose.Evaluate(Output);

		// Restore proxy & required bones after evaluation
		Output.AnimInstanceProxy = &OldProxy;
		Output.Pose.SetBoneContainer(&OldProxy.GetRequiredBones());
	}
	else if(CachedInputPose.IsValid() && ensure(Output.Pose.GetNumBones() == CachedInputPose.GetNumBones()) && bIsCachedInputPoseInitialized)
	{
		Output.Pose.CopyBonesFrom(CachedInputPose);
		Output.Curve.CopyFrom(CachedInputCurve);
		Output.CustomAttributes.CopyFrom(CachedAttributes);
	}
	else
	{
		Output.ResetToRefPose();
	}
}

void FAnimNode_LinkedInputPose::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugData.AddDebugItem(DebugLine);

	if(InputProxy)
	{
		InputPose.GatherDebugData(DebugData);
	}
}

void FAnimNode_LinkedInputPose::DynamicLink(FAnimInstanceProxy* InInputProxy, FPoseLinkBase* InPoseLink, int32 InOuterGraphNodeIndex)
{
	ensure(GIsReinstancing || InputProxy == nullptr);			// Must be unlinked before re-linking

	InputProxy = InInputProxy;
	InputPose.SetDynamicLinkNode(InPoseLink);
	OuterGraphNodeIndex = InOuterGraphNodeIndex;
}

void FAnimNode_LinkedInputPose::DynamicUnlink()
{
	ensure(GIsReinstancing || InputProxy != nullptr);			// Must be linked before unlinking

	InputProxy = nullptr;
	InputPose.SetDynamicLinkNode(nullptr);
	OuterGraphNodeIndex = INDEX_NONE;
}
