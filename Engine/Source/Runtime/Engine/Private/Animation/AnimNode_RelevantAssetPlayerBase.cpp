// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_RelevantAssetPlayerBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RelevantAssetPlayerBase)

UAnimationAsset* FAnimNode_AssetPlayerRelevancyBase::GetAnimAsset() const
{
	return nullptr;
}

float FAnimNode_AssetPlayerRelevancyBase::GetAccumulatedTime() const
{
	return 0.f;
}

void FAnimNode_AssetPlayerRelevancyBase::SetAccumulatedTime(float NewTime)
{
}

float FAnimNode_AssetPlayerRelevancyBase::GetCurrentAssetLength() const
{
	return 0.f;
}

float FAnimNode_AssetPlayerRelevancyBase::GetCurrentAssetTime() const
{
	return 0.f;
}

float FAnimNode_AssetPlayerRelevancyBase::GetCurrentAssetTimePlayRateAdjusted() const
{
	return 0.f;
}

bool FAnimNode_AssetPlayerRelevancyBase::IsLooping() const
{
	return true;
}

bool FAnimNode_AssetPlayerRelevancyBase::GetIgnoreForRelevancyTest() const
{
	return false;
}

bool FAnimNode_AssetPlayerRelevancyBase::SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest)
{
	return false;
}

float FAnimNode_AssetPlayerRelevancyBase::GetCachedBlendWeight() const
{
	return 0.f;
}

void FAnimNode_AssetPlayerRelevancyBase::ClearCachedBlendWeight()
{
}

const FDeltaTimeRecord* FAnimNode_AssetPlayerRelevancyBase::GetDeltaTimeRecord() const
{
	return nullptr;
}
