// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_Root.h"
#include "Animation/AnimTrace.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Root)

/////////////////////////////////////////////////////
// FAnimNode_Root

#if WITH_EDITORONLY_DATA
FName FAnimNode_Root::DefaultSharedGroup("DefaultSharedGroup"); // All layers sharing the instance by default
#endif

FAnimNode_Root::FAnimNode_Root()
{
}

void FAnimNode_Root::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Result.Initialize(Context);
}

void FAnimNode_Root::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) 
{
	Result.CacheBones(Context);
}

void FAnimNode_Root::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), GetName());

	Result.Update(Context);
}

void FAnimNode_Root::Evaluate_AnyThread(FPoseContext& Output)
{
	Result.Evaluate(Output);
}

void FAnimNode_Root::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugData.AddDebugItem(DebugLine);
	Result.GatherDebugData(DebugData);
}

FName FAnimNode_Root::GetName() const
{
	return GET_ANIM_NODE_DATA(FName, Name);
}

FName FAnimNode_Root::GetGroup() const
{
	return GET_ANIM_NODE_DATA(FName, LayerGroup);
}
