// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_SaveCachedPose.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_SaveCachedPose)

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::Anim::FCachedPoseSkippedUpdateHandler);

namespace UE::Anim
{

/** Holds references to scoped cached poses to ensure lifetime is correct */
struct FCachedPoseThreadContext : TThreadSingleton<FCachedPoseThreadContext>
{
	/** Cached pose - note that all memory is allocated on the current mem stack allocator */
	struct FCachedPose
	{
		FCompactPose Pose;
		FBlendedCurve Curve;
		UE::Anim::FStackAttributeContainer Attributes;
	};

	/** Cached pose scope */
	struct FCachedPoseScopeInternal
	{
		// Map of nodes to cached poses
		TMap<const FAnimNode_SaveCachedPose*, FCachedPose, TInlineSetAllocator<32>> CachedPoses;
	};

	/** Adds a cached pose */
	const FCachedPose& AddCachedPose(const FAnimNode_SaveCachedPose* InNode, FCompactPose& InPose, FBlendedCurve& InCurve, UE::Anim::FStackAttributeContainer& InAttributes)
	{
		check(Scopes.Num() > 0);

		FCachedPose& CachedPose = Scopes.Top().CachedPoses.Add(InNode);

		CachedPose.Pose.MoveBonesFrom(InPose);
		CachedPose.Curve.MoveFrom(InCurve);
		CachedPose.Attributes.MoveFrom(InAttributes);

		return CachedPose;
	}

	/** Finds a cached pose */
	const FCachedPose* FindCachedPose(const FAnimNode_SaveCachedPose* InNode) const
	{
		if(Scopes.Num() > 0)
		{
			return Scopes.Top().CachedPoses.Find(InNode);
		}

		return nullptr;
	}

	/** All cached pose scopes for the current thread */
	TArray<FCachedPoseScopeInternal, TInlineAllocator<1>> Scopes;
};

FCachedPoseScope::FCachedPoseScope()
{
	FCachedPoseThreadContext::Get().Scopes.Push(FCachedPoseThreadContext::FCachedPoseScopeInternal());
}

FCachedPoseScope::~FCachedPoseScope()
{
	FCachedPoseThreadContext::Get().Scopes.Pop();
}

}

/////////////////////////////////////////////////////
// FAnimNode_SaveCachedPose

FAnimNode_SaveCachedPose::FAnimNode_SaveCachedPose()
	: GlobalWeight(0.0f)
{
}

void FAnimNode_SaveCachedPose::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	// StateMachines cause reinitialization on state changes.
	// we only want to let them through if we're not relevant as to not create a pop.
	if (!InitializationCounter.IsSynchronized_Counter(Context.AnimInstanceProxy->GetInitializationCounter())
		|| (UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter())))
	{
		InitializationCounter.SynchronizeWith(Context.AnimInstanceProxy->GetInitializationCounter());

		FAnimNode_Base::Initialize_AnyThread(Context);

		// Initialize the subgraph
		Pose.Initialize(Context);
	}
}

void FAnimNode_SaveCachedPose::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	if (!CachedBonesCounter.IsSynchronized_All(Context.AnimInstanceProxy->GetCachedBonesCounter()))
	{
		CachedBonesCounter.SynchronizeWith(Context.AnimInstanceProxy->GetCachedBonesCounter());

		// Pose will be out of date, so reset the evaluation counter
		EvaluationCounter.Reset();
		
		// Cache bones in the subgraph
		Pose.CacheBones(Context);
	}
}

void FAnimNode_SaveCachedPose::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	FCachedUpdateContext& CachedUpdate = CachedUpdateContexts.AddDefaulted_GetRef();

	// Make a minimal copy of the shared context for cached updates
	if (FAnimationUpdateSharedContext* SharedContext = Context.GetSharedContext())
	{
		CachedUpdate.SharedContext = MakeUnique<FAnimationUpdateSharedContext>();
		CachedUpdate.SharedContext->CopyForCachedUpdate(*SharedContext);
	}

	// Store this context for the post update
	CachedUpdate.Context = Context.WithOtherSharedContext(CachedUpdate.SharedContext.Get());

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), CachePoseName);
}

void FAnimNode_SaveCachedPose::Evaluate_AnyThread(FPoseContext& Output)
{
	using namespace UE::Anim;

	FCachedPoseThreadContext& CachedPoseThreadContext = FCachedPoseThreadContext::Get();
	const FCachedPoseThreadContext::FCachedPose* CachedPose = nullptr;

	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(SaveCachedPose, !IsInGameThread());

	// Note that we check here for IsSynchronized_All to deal with cases like Sequencer:
	// In these cases the counter can stay zeroed between unbound/bound updates and this can cause issues
	// with using out-of-date cached data (stack-allocated from a previous frame).
	const bool bSynchronized = EvaluationCounter.IsSynchronized_All(Output.AnimInstanceProxy->GetEvaluationCounter());
	if(bSynchronized)
	{
		// Synchronized, so check whether we have a cached curve to use
		CachedPose = CachedPoseThreadContext.FindCachedPose(this);
	}

	const bool bShouldCachePose = !bSynchronized || CachedPose == nullptr;

	if (bShouldCachePose)
	{
		EvaluationCounter.SynchronizeWith(Output.AnimInstanceProxy->GetEvaluationCounter());

		FPoseContext CachingContext(Output);
		Pose.Evaluate(CachingContext);

		CachedPose = &CachedPoseThreadContext.AddCachedPose(this, CachingContext.Pose, CachingContext.Curve, CachingContext.CustomAttributes);
	}

	// Return the cached result
	Output.Pose.CopyBonesFrom(CachedPose->Pose);
	Output.Curve.CopyFrom(CachedPose->Curve);
	Output.CustomAttributes.CopyFrom(CachedPose->Attributes);
}

void FAnimNode_SaveCachedPose::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("%s:"), *CachePoseName.ToString());

	if (FNodeDebugData* SaveCachePoseDebugDataPtr = DebugData.GetCachePoseDebugData(GlobalWeight))
	{
		SaveCachePoseDebugDataPtr->AddDebugItem(DebugLine);
		Pose.GatherDebugData(*SaveCachePoseDebugDataPtr);
	}
}

void FAnimNode_SaveCachedPose::PostGraphUpdate()
{
	GlobalWeight = 0.f;

	// Update GlobalWeight based on highest weight calling us.
	const int32 NumContexts = CachedUpdateContexts.Num();
	if (NumContexts > 0)
	{
		GlobalWeight = CachedUpdateContexts[0].Context.GetFinalBlendWeight();
		int32 MaxWeightIdx = 0;
		for (int32 CurrIdx = 1; CurrIdx < NumContexts; ++CurrIdx)
		{
			const float BlendWeight = CachedUpdateContexts[CurrIdx].Context.GetFinalBlendWeight();
			if (BlendWeight > GlobalWeight)
			{
				GlobalWeight = BlendWeight;
				MaxWeightIdx = CurrIdx;
			}
		}

		// Update the max weighted pose node
		{
			TRACE_SCOPED_ANIM_NODE(CachedUpdateContexts[MaxWeightIdx].Context);
			Pose.Update(CachedUpdateContexts[MaxWeightIdx].Context);
		}

		// Update any branches that will be skipped
		if (CachedUpdateContexts[MaxWeightIdx].SharedContext)
		{
			UE::Anim::FMessageStack& MessageStack = CachedUpdateContexts[MaxWeightIdx].SharedContext->MessageStack;
			if (MessageStack.HasMessage<UE::Anim::FCachedPoseSkippedUpdateHandler>())
			{
				// Grab handles to all the execution paths that we are not proceeding with
				TArray<UE::Anim::FMessageStack, TInlineAllocator<4>> SkippedMessageStacks;
				for (int32 CurrIdx = 0; CurrIdx < NumContexts; ++CurrIdx)
				{
					if (CurrIdx != MaxWeightIdx && CachedUpdateContexts[CurrIdx].SharedContext)
					{
						SkippedMessageStacks.Add(MoveTemp(CachedUpdateContexts[CurrIdx].SharedContext->MessageStack));
					}
				}

				// Broadcast skipped update message to interested parties
				MessageStack.ForEachMessage<UE::Anim::FCachedPoseSkippedUpdateHandler>([&SkippedMessageStacks](UE::Anim::FCachedPoseSkippedUpdateHandler& InMessage)
					{
						// Fire the event to inform listeners of 'skipped' paths
						InMessage.OnUpdatesSkipped(SkippedMessageStacks);

						// We only want the topmost registered node here, so early out
						return UE::Anim::FMessageStack::EEnumerate::Stop;
					});
			}
		}
	}

	CachedUpdateContexts.Reset();
}

