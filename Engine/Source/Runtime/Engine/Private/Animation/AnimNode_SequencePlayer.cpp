// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_SequencePlayer.h"

#include "Animation/AnimMontage.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AnimPoseSearchProvider.h"
#include "Animation/ExposedValueHandler.h"
#include "Logging/TokenizedMessage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_SequencePlayer)

#if WITH_EDITORONLY_DATA
#include "Animation/AnimBlueprintGeneratedClass.h"
#endif

#define LOCTEXT_NAMESPACE "AnimNode_SequencePlayer"

/////////////////////////////////////////////////////
// FAnimSequencePlayerNode

float FAnimNode_SequencePlayerBase::GetCurrentAssetTime() const
{
	return InternalTimeAccumulator;
}

float FAnimNode_SequencePlayerBase::GetCurrentAssetTimePlayRateAdjusted() const
{
	UAnimSequenceBase* CurrentSequence = GetSequence();
	const float SequencePlayRate = (CurrentSequence ? CurrentSequence->RateScale : 1.f);
	const float CurrentPlayRate = GetPlayRate();
	const float CurrentPlayRateBasis = GetPlayRateBasis();

	const float AdjustedPlayRate = PlayRateScaleBiasClampState.ApplyTo(GetPlayRateScaleBiasClampConstants(), FMath::IsNearlyZero(CurrentPlayRateBasis) ? 0.f : (CurrentPlayRate / CurrentPlayRateBasis));
	const float EffectivePlayrate = SequencePlayRate * AdjustedPlayRate;
	return (EffectivePlayrate < 0.0f) ? GetCurrentAssetLength() - InternalTimeAccumulator : InternalTimeAccumulator;
}

float FAnimNode_SequencePlayerBase::GetCurrentAssetLength() const
{
	UAnimSequenceBase* CurrentSequence = GetSequence();
	return CurrentSequence ? CurrentSequence->GetPlayLength() : 0.0f;
}

void FAnimNode_SequencePlayerBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread);

	FAnimNode_AssetPlayerBase::Initialize_AnyThread(Context);

	GetEvaluateGraphExposedInputs().Execute(Context);

	UAnimSequenceBase* CurrentSequence = GetSequence();
	if (CurrentSequence && !ensureMsgf(!CurrentSequence->IsA<UAnimMontage>(), TEXT("Sequence players do not support anim montages.")))
	{
		CurrentSequence = nullptr;
	}

	InternalTimeAccumulator = GetStartPosition();
	PlayRateScaleBiasClampState.Reinitialize();

	if (CurrentSequence != nullptr)
	{
		const float EffectiveStartPosition = GetEffectiveStartPosition(Context);
		const float CurrentPlayRate = GetPlayRate();
		const float CurrentPlayRateBasis = GetPlayRateBasis();

		InternalTimeAccumulator = FMath::Clamp(EffectiveStartPosition, 0.f, CurrentSequence->GetPlayLength());
		const float AdjustedPlayRate = PlayRateScaleBiasClampState.ApplyTo(GetPlayRateScaleBiasClampConstants(), FMath::IsNearlyZero(CurrentPlayRateBasis) ? 0.f : (CurrentPlayRate / CurrentPlayRateBasis), 0.f);
		const float EffectivePlayrate = CurrentSequence->RateScale * AdjustedPlayRate;
		if ((EffectiveStartPosition == 0.f) && (EffectivePlayrate < 0.f))
		{
			InternalTimeAccumulator = CurrentSequence->GetPlayLength();
		}
	}
}

void FAnimNode_SequencePlayerBase::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread);
}

void FAnimNode_SequencePlayerBase::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateAssetPlayer);

	GetEvaluateGraphExposedInputs().Execute(Context);

	UAnimSequenceBase* CurrentSequence = GetSequence();
	if (CurrentSequence && !ensureMsgf(!CurrentSequence->IsA<UAnimMontage>(), TEXT("Sequence players do not support anim montages.")))
	{
		CurrentSequence = nullptr;
	}

	if (CurrentSequence != nullptr && CurrentSequence->GetSkeleton() != nullptr)
	{
		const float CurrentPlayRate = GetPlayRate();
		const float CurrentPlayRateBasis = GetPlayRateBasis();

		InternalTimeAccumulator = FMath::Clamp(InternalTimeAccumulator, 0.f, CurrentSequence->GetPlayLength());
		const float AdjustedPlayRate = PlayRateScaleBiasClampState.ApplyTo(GetPlayRateScaleBiasClampConstants(), FMath::IsNearlyZero(CurrentPlayRateBasis) ? 0.f : (CurrentPlayRate / CurrentPlayRateBasis), Context.GetDeltaTime());
		CreateTickRecordForNode(Context, CurrentSequence, IsLooping(), AdjustedPlayRate, false);
	}

#if WITH_EDITORONLY_DATA
	if (FAnimBlueprintDebugData* DebugData = Context.AnimInstanceProxy->GetAnimBlueprintDebugData())
	{
		DebugData->RecordSequencePlayer(Context.GetCurrentNodeId(), GetAccumulatedTime(), CurrentSequence != nullptr ? CurrentSequence->GetPlayLength() : 0.0f, CurrentSequence != nullptr ? CurrentSequence->GetNumberOfSampledKeys() : 0);
	}
#endif

	TRACE_ANIM_SEQUENCE_PLAYER(Context, *this);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), CurrentSequence != nullptr ? CurrentSequence->GetFName() : NAME_None);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Asset"), CurrentSequence);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Playback Time"), InternalTimeAccumulator);
}

void FAnimNode_SequencePlayerBase::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread);

	UAnimSequenceBase* CurrentSequence = GetSequence();
	if (CurrentSequence != nullptr && CurrentSequence->GetSkeleton() != nullptr)
	{
		const bool bExpectedAdditive = Output.ExpectsAdditivePose();
		const bool bIsAdditive = CurrentSequence->IsValidAdditive();

		if (bExpectedAdditive && !bIsAdditive)
		{
			FText Message = FText::Format(LOCTEXT("AdditiveMismatchWarning", "Trying to play a non-additive animation '{0}' into a pose that is expected to be additive in anim instance '{1}'"), FText::FromString(CurrentSequence->GetName()), FText::FromString(Output.AnimInstanceProxy->GetAnimInstanceName()));
			Output.LogMessage(EMessageSeverity::Warning, Message);
		}

		FAnimationPoseData AnimationPoseData(Output);
		CurrentSequence->GetAnimationPose(AnimationPoseData, FAnimExtractContext(static_cast<double>(InternalTimeAccumulator), Output.AnimInstanceProxy->ShouldExtractRootMotion(), DeltaTimeRecord, IsLooping()));
	}
	else
	{
		Output.ResetToRefPose();
	}
}

void FAnimNode_SequencePlayerBase::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);

	UAnimSequenceBase* CurrentSequence = GetSequence();
	DebugLine += FString::Printf(TEXT("('%s' Play Time: %.3f)"), CurrentSequence ? *CurrentSequence->GetName() : TEXT("NULL"), InternalTimeAccumulator);
	DebugData.AddDebugItem(DebugLine, true);
}

float FAnimNode_SequencePlayerBase::GetTimeFromEnd(float CurrentNodeTime) const
{
	UAnimSequenceBase* CurrentSequence = GetSequence();
	return CurrentSequence ? CurrentSequence->GetPlayLength() - CurrentNodeTime : 0.0f;
}

float FAnimNode_SequencePlayerBase::GetEffectiveStartPosition(const FAnimationBaseContext& Context) const
{
	using namespace UE::Anim;

	// Override the start position if pose matching is enabled
	const UObject* CurrentSequence = GetSequence();
	if (CurrentSequence != nullptr && GetStartFromMatchingPose())
	{
		if (const IPoseSearchProvider* PoseSearchProvider = IPoseSearchProvider::Get())
		{
			const IPoseSearchProvider::FSearchResult Result = PoseSearchProvider->Search(Context, MakeArrayView(&CurrentSequence, 1),
				IPoseSearchProvider::FSearchPlayingAsset(), IPoseSearchProvider::FSearchFutureAsset());
			if (Result.SelectedAsset != nullptr)
			{
				return Result.TimeOffsetSeconds;
			}
		}
	}

	return GetStartPosition();
}

bool FAnimNode_SequencePlayer::SetSequence(UAnimSequenceBase* InSequence)
{
	Sequence = InSequence;
	return true;
}

bool FAnimNode_SequencePlayer::SetLoopAnimation(bool bInLoopAnimation)
{
#if WITH_EDITORONLY_DATA
	bLoopAnimation = bInLoopAnimation;
#endif
	
	if(bool* bLoopAnimationPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bLoopAnimation))
	{
		*bLoopAnimationPtr = bInLoopAnimation;
		return true;
	}

	return false;
}

UAnimSequenceBase* FAnimNode_SequencePlayer::GetSequence() const
{
	return Sequence;
}

float FAnimNode_SequencePlayer::GetPlayRateBasis() const
{
	return GET_ANIM_NODE_DATA(float, PlayRateBasis);
}

float FAnimNode_SequencePlayer::GetPlayRate() const
{
	return GET_ANIM_NODE_DATA(float, PlayRate);
}

const FInputScaleBiasClampConstants& FAnimNode_SequencePlayer::GetPlayRateScaleBiasClampConstants() const
{
	return GET_ANIM_NODE_DATA(FInputScaleBiasClampConstants, PlayRateScaleBiasClampConstants);
}

float FAnimNode_SequencePlayer::GetStartPosition() const
{
	return GET_ANIM_NODE_DATA(float, StartPosition);
}

bool FAnimNode_SequencePlayer::IsLooping() const
{
	return GET_ANIM_NODE_DATA(bool, bLoopAnimation);
}

bool FAnimNode_SequencePlayer::GetStartFromMatchingPose() const
{
	return GET_ANIM_NODE_DATA(bool, bStartFromMatchingPose);
}

FName FAnimNode_SequencePlayer::GetGroupName() const
{
	return GET_ANIM_NODE_DATA(FName, GroupName);
}

EAnimGroupRole::Type FAnimNode_SequencePlayer::GetGroupRole() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<EAnimGroupRole::Type>, GroupRole);
}

bool FAnimNode_SequencePlayer::GetOverridePositionWhenJoiningSyncGroupAsLeader() const
{
	return GET_ANIM_NODE_DATA(bool, bOverridePositionWhenJoiningSyncGroupAsLeader);
}

EAnimSyncMethod FAnimNode_SequencePlayer::GetGroupMethod() const
{
	return GET_ANIM_NODE_DATA(EAnimSyncMethod, Method);
}

bool FAnimNode_SequencePlayer::GetIgnoreForRelevancyTest() const
{
	return GET_ANIM_NODE_DATA(bool, bIgnoreForRelevancyTest);
}

bool FAnimNode_SequencePlayer::SetGroupName(FName InGroupName)
{
#if WITH_EDITORONLY_DATA
	GroupName = InGroupName;
#endif

	if(FName* GroupNamePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(FName, GroupName))
	{
		*GroupNamePtr = InGroupName;
		return true;
	}

	return false;
}

bool FAnimNode_SequencePlayer::SetGroupRole(EAnimGroupRole::Type InRole)
{
#if WITH_EDITORONLY_DATA
	GroupRole = InRole;
#endif

	if(TEnumAsByte<EAnimGroupRole::Type>* GroupRolePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(TEnumAsByte<EAnimGroupRole::Type>, GroupRole))
	{
		*GroupRolePtr = InRole;
		return true;
	}

	return false;
}

bool FAnimNode_SequencePlayer::SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader)
{
#if WITH_EDITORONLY_DATA
	bOverridePositionWhenJoiningSyncGroupAsLeader = InOverridePositionWhenJoiningSyncGroupAsLeader;
#endif

	if (bool* OverridePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bOverridePositionWhenJoiningSyncGroupAsLeader))
	{
		*OverridePtr = InOverridePositionWhenJoiningSyncGroupAsLeader;
		return true;
	}

	return false;
}

bool FAnimNode_SequencePlayer::SetGroupMethod(EAnimSyncMethod InMethod)
{
#if WITH_EDITORONLY_DATA
	Method = InMethod;
#endif

	if(EAnimSyncMethod* MethodPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(EAnimSyncMethod, Method))
	{
		*MethodPtr = InMethod;
		return true;
	}

	return false;
}

bool FAnimNode_SequencePlayer::SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest)
{
#if WITH_EDITORONLY_DATA
	bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest;
#endif

	if(bool* bIgnoreForRelevancyTestPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bIgnoreForRelevancyTest))
	{
		*bIgnoreForRelevancyTestPtr = bInIgnoreForRelevancyTest;
		return true;
	}

	return false;
}

bool FAnimNode_SequencePlayer::SetStartPosition(float InStartPosition)
{
	if(float* StartPositionPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(float, StartPosition))
	{
		*StartPositionPtr = InStartPosition;
		return true;
	}

	return false;
}


bool FAnimNode_SequencePlayer::SetPlayRate(float InPlayRate)
{
	if(float* PlayRatePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(float, PlayRate))
	{
		*PlayRatePtr = InPlayRate;
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

