// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_StateMachine.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNode_RelevantAssetPlayerBase.h"
#include "Animation/AnimNode_TransitionResult.h"
#include "Animation/AnimNode_TransitionPoseEvaluator.h"
#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "Animation/BlendProfile.h"
#include "Animation/AnimNode_LinkedAnimLayer.h"
#include "Animation/ActiveStateMachineScope.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/ExposedValueHandler.h"
#include "Logging/TokenizedMessage.h"
#include "Animation/AnimInertializationSyncScope.h"
#include "Animation/AnimNode_StateResult.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "HAL/IConsoleManager.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_StateMachine)

#if WITH_EDITORONLY_DATA
#endif

#define LOCTEXT_NAMESPACE "AnimNode_StateMachine"

DEFINE_LOG_CATEGORY_STATIC(LogAnimTransitionRequests, NoLogging, All);

DECLARE_CYCLE_STAT(TEXT("StateMachine SetState"), Stat_StateMachineSetState, STATGROUP_Anim);

static const FName DefaultAnimGraphName("AnimGraph");

//////////////////////////////////////////////////////////////////////////
// FAnimationActiveTransitionEntry

FAnimationActiveTransitionEntry::FAnimationActiveTransitionEntry()
	: ElapsedTime(0.0f)
	, Alpha(0.0f)
	, CrossfadeDuration(0.0f)
	, NextState(INDEX_NONE)
	, PreviousState(INDEX_NONE)
	, StartNotify(INDEX_NONE)
	, EndNotify(INDEX_NONE)
	, InterruptNotify(INDEX_NONE)
	, BlendProfile(nullptr)
	, BlendOption(EAlphaBlendOption::HermiteCubic)
	, LogicType(ETransitionLogicType::TLT_StandardBlend)
	, bActive(false)
{
}

FAnimationActiveTransitionEntry::FAnimationActiveTransitionEntry(int32 NextStateID, float ExistingWeightOfNextState, int32 PreviousStateID, const FAnimationTransitionBetweenStates& ReferenceTransitionInfo, float CrossfadeTimeAdjustment)
	: ElapsedTime(0.0f)
	, Alpha(0.0f)
	, NextState(NextStateID)
	, PreviousState(PreviousStateID)
	, StartNotify(ReferenceTransitionInfo.StartNotify)
	, EndNotify(ReferenceTransitionInfo.EndNotify)
	, InterruptNotify(ReferenceTransitionInfo.InterruptNotify)
	, BlendProfile(ReferenceTransitionInfo.BlendProfile)
	, BlendOption(ReferenceTransitionInfo.BlendMode)
	, LogicType(ReferenceTransitionInfo.LogicType)
	, bActive(true)
{
	const float Scaler = 1.0f - ExistingWeightOfNextState;
	CrossfadeDuration = (LogicType == ETransitionLogicType::TLT_Inertialization) ? 0.0f : FMath::Max(ReferenceTransitionInfo.CrossfadeDuration - CrossfadeTimeAdjustment, 0.f) * CalculateInverseAlpha(BlendOption, Scaler);

	Blend.SetBlendTime(CrossfadeDuration);
	Blend.SetBlendOption(BlendOption);
	Blend.SetCustomCurve(ReferenceTransitionInfo.CustomCurve);
	Blend.SetValueRange(0.0f, 1.0f);
}

float FAnimationActiveTransitionEntry::CalculateInverseAlpha(EAlphaBlendOption BlendMode, float InFraction) const
{
	if(BlendMode == EAlphaBlendOption::HermiteCubic)
	{
		const float A = 4.0f / 3.0f;
		const float B = -2.0f;
		const float C = 5.0f / 3.0f;

		const float T = InFraction;
		const float TT = InFraction*InFraction;
		const float TTT = InFraction*InFraction*InFraction;

		return TTT*A + TT*B + T*C;
	}
	else
	{
		return FMath::Clamp<float>(InFraction, 0.0f, 1.0f);
	}
}

void FAnimationActiveTransitionEntry::InitializeCustomGraphLinks(const FAnimationUpdateContext& Context, const FBakedStateExitTransition& TransitionRule)
{
	if (TransitionRule.CustomResultNodeIndex != INDEX_NONE)
	{
		if (const IAnimClassInterface* AnimBlueprintClass = Context.GetAnimClass())
		{
			CustomTransitionGraph.LinkID = AnimBlueprintClass->GetAnimNodeProperties().Num() - 1 - TransitionRule.CustomResultNodeIndex; //@TODO: Crazysauce
			FAnimationInitializeContext InitContext(Context.AnimInstanceProxy, Context.SharedContext);
			CustomTransitionGraph.Initialize(InitContext);

			if (Context.AnimInstanceProxy)
			{
				for (int32 Index = 0; Index < TransitionRule.PoseEvaluatorLinks.Num(); ++Index)
				{
					FAnimNode_TransitionPoseEvaluator* PoseEvaluator = GetNodeFromPropertyIndex<FAnimNode_TransitionPoseEvaluator>(Context.AnimInstanceProxy->GetAnimInstanceObject(), AnimBlueprintClass, TransitionRule.PoseEvaluatorLinks[Index]);
					PoseEvaluators.Add(PoseEvaluator);
				}
			}
		}
	}
}

void FAnimationActiveTransitionEntry::Update(const FAnimationUpdateContext& Context, int32 CurrentStateIndex, bool& bOutFinished)
{
	bOutFinished = false;

	// Advance time
	if (bActive)
	{
		ElapsedTime += Context.GetDeltaTime();
		Blend.Update(Context.GetDeltaTime());

		// If non-zero, calculate the query alpha
		float QueryAlpha = 0.0f;
		if (CrossfadeDuration > 0.0f)
		{
			QueryAlpha = ElapsedTime / CrossfadeDuration;
		}

		Alpha = FAlphaBlend::AlphaToBlendOption(QueryAlpha, Blend.GetBlendOption(), Blend.GetCustomCurve());

		if (Blend.IsComplete())
		{
			bActive = false;
			bOutFinished = true;
		}

		// Update state blend data (only when we're using per-bone)
		if (BlendProfile)
		{
			for (int32 Idx = 0 ; Idx < 2 ; ++Idx)
			{
				const bool bForwards = (Idx == 0);
				StateBlendData[Idx].TotalWeight = bForwards ? Alpha : 1.0f - Alpha;
				BlendProfile->UpdateBoneWeights(StateBlendData[Idx], Blend, 0.0f, StateBlendData[Idx].TotalWeight, !bForwards);
			}

			FBlendSampleData::NormalizeDataWeight(StateBlendData);
		}
	}
}

bool FAnimationActiveTransitionEntry::Serialize(FArchive& Ar)
{
	Ar << ElapsedTime;
	Ar << Alpha;
	Ar << CrossfadeDuration;
	Ar << bActive;
	Ar << NextState;
	Ar << PreviousState;

	return true;
}

/////////////////////////////////////////////////////
// FAnimationPotentialTransition

FAnimationPotentialTransition::FAnimationPotentialTransition()
: 	TargetState(INDEX_NONE)
,	CrossfadeTimeAdjustment(0.f)
,	TransitionRule(NULL)
{
}

bool FAnimationPotentialTransition::IsValid() const
{
	return (TargetState != INDEX_NONE) && (TransitionRule != NULL) && (TransitionRule->TransitionIndex != INDEX_NONE);
}

void FAnimationPotentialTransition::Clear()
{
	TargetState = INDEX_NONE;
	CrossfadeTimeAdjustment = 0.f;
	TransitionRule = NULL;
	SourceTransitionIndices.Reset();
}


/////////////////////////////////////////////////////
// FAnimNode_StateMachine

// Tries to get the instance information for the state machine
const FBakedAnimationStateMachine* FAnimNode_StateMachine::GetMachineDescription() const
{
	if (PRIVATE_MachineDescription != nullptr)
	{
		return PRIVATE_MachineDescription;
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "FAnimNode_StateMachine: Bad machine ptr");
		return nullptr;
	}
}

void FAnimNode_StateMachine::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_Base::Initialize_AnyThread(Context);

	IAnimClassInterface* AnimBlueprintClass = Context.GetAnimClass();

	if (const FBakedAnimationStateMachine* Machine = GetMachineDescription())
	{
		ElapsedTime = 0.0f;

		CurrentState = INDEX_NONE;

		if (Machine->States.Num() > 0)
		{
			// Create a pose link for each state we can reach
			StatePoseLinks.Reset();
			StatePoseLinks.Reserve(Machine->States.Num());
			for (int32 StateIndex = 0; StateIndex < Machine->States.Num(); ++StateIndex)
			{
				const FBakedAnimationState& State = Machine->States[StateIndex];
				FPoseLink& StatePoseLink = StatePoseLinks.AddDefaulted_GetRef();

				// because conduits don't contain bound graphs, this link is no longer guaranteed to be valid
				if (State.StateRootNodeIndex != INDEX_NONE)
				{
					StatePoseLink.LinkID = AnimBlueprintClass->GetAnimNodeProperties().Num() - 1 - State.StateRootNodeIndex; //@TODO: Crazysauce
				}

				// also initialize transitions
				if(State.EntryRuleNodeIndex != INDEX_NONE)
				{
					if (FAnimNode_TransitionResult* TransitionNode = GetNodeFromPropertyIndex<FAnimNode_TransitionResult>(Context.AnimInstanceProxy->GetAnimInstanceObject(), AnimBlueprintClass, State.EntryRuleNodeIndex))
					{
						TransitionNode->Initialize_AnyThread(Context);
					}
				}

				for(int32 TransitionIndex = 0; TransitionIndex < State.Transitions.Num(); ++TransitionIndex)
				{
					const FBakedStateExitTransition& TransitionRule = State.Transitions[TransitionIndex];
					if (TransitionRule.CanTakeDelegateIndex != INDEX_NONE)
					{
						if (FAnimNode_TransitionResult* TransitionNode = GetNodeFromPropertyIndex<FAnimNode_TransitionResult>(Context.AnimInstanceProxy->GetAnimInstanceObject(), AnimBlueprintClass, TransitionRule.CanTakeDelegateIndex))
						{
							TransitionNode->Initialize_AnyThread(Context);
						}
					}
				}
			}

			// Reset transition related variables
			StatesUpdated.Reset();
			ActiveTransitionArray.Reset();
			QueuedTransitionEvents.Reset();

			StateCacheBoneCounters.Reset(Machine->States.Num());
			StateCacheBoneCounters.AddDefaulted(Machine->States.Num());

			LastStateEntryTime.Reset(Machine->States.Num());
			LastStateEntryTime.AddZeroed(Machine->States.Num());

			// Move to the default state
			SetState(Context, Machine->InitialState, false);

			// initialize first update
			bFirstUpdate = true;
		}
	}
}

void FAnimNode_StateMachine::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) 
{
	if (const FBakedAnimationStateMachine* Machine = GetMachineDescription())
	{
		for (int32 StateIndex = 0; StateIndex < Machine->States.Num(); ++StateIndex)
		{
			if (GetStateWeight(StateIndex) > 0.f)
			{
				ConditionallyCacheBonesForState(StateIndex, Context);
			}
		}
	}

	// @TODO GetStateWeight is O(N) transitions.
}

void FAnimNode_StateMachine::ConditionallyCacheBonesForState(int32 StateIndex, FAnimationBaseContext Context)
{
	// Only call CacheBones when needed.
	check(StateCacheBoneCounters.IsValidIndex(StateIndex));

	const FGraphTraversalCounter& ProxyCachedBonesCounter = Context.AnimInstanceProxy->GetCachedBonesCounter();

	// Check both the frame and counter
	// Multiple anim instances can be linked together and their LODs can be independent
	// This means that a sub-instance can have its LOD change, causing the value we cache here to run ahead
	// of the one from the main anim instance. When the LOD of the main instance will change, we'll reset
	// all the counters to match it which can cause them to run backwards
	if (!StateCacheBoneCounters[StateIndex].IsSynchronized_All(ProxyCachedBonesCounter))
	{
		// keep track of states that have had CacheBones called on.
		StateCacheBoneCounters[StateIndex].SynchronizeWith(ProxyCachedBonesCounter);

		FAnimationCacheBonesContext CacheBoneContext(Context.AnimInstanceProxy);
		StatePoseLinks[StateIndex].CacheBones(CacheBoneContext);
	}
}

const FBakedAnimationState& FAnimNode_StateMachine::GetStateInfo() const
{
	return PRIVATE_MachineDescription->States[CurrentState];
}

const FBakedAnimationState& FAnimNode_StateMachine::GetStateInfo(int32 StateIndex) const
{
	return PRIVATE_MachineDescription->States[StateIndex];
}

const int32 FAnimNode_StateMachine::GetStateIndex( const FBakedAnimationState& StateInfo ) const
{
	for (int32 Index = 0; Index < PRIVATE_MachineDescription->States.Num(); ++Index)
	{
		if( &PRIVATE_MachineDescription->States[Index] == &StateInfo )
		{
			return Index;
		}
	}

	return INDEX_NONE;
}


const int32 FAnimNode_StateMachine::GetStateIndex(FName StateName) const
{
	for (int32 Index = 0; Index < PRIVATE_MachineDescription->States.Num(); ++Index)
	{
		if (PRIVATE_MachineDescription->States[Index].StateName == StateName)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

const FAnimationTransitionBetweenStates& FAnimNode_StateMachine::GetTransitionInfo(int32 TransIndex) const
{
	return PRIVATE_MachineDescription->Transitions[TransIndex];
}

void FAnimNode_StateMachine::LogInertializationRequestError(const FAnimationUpdateContext& Context, int32 PreviousState, int32 NextState)
{
#if WITH_EDITORONLY_DATA
	const FBakedAnimationStateMachine* Machine = GetMachineDescription();
	FText Message = FText::Format(LOCTEXT("InertialTransitionError", "No Inertialization node found for request from transition '{0}' to '{1}' in state machine '{2}' in anim blueprint '{3}'. Add an Inertialization node after this state machine."),
		FText::FromName(GetStateInfo(PreviousState).StateName),
		FText::FromName(GetStateInfo(NextState).StateName),
		FText::FromName(Machine->MachineName),
		FText::FromString(GetPathNameSafe(Context.AnimInstanceProxy->GetAnimBlueprint())));
	Context.LogMessage(EMessageSeverity::Error, Message);
#endif
}

// Temporarily turned off while we track down and fix https://jira.ol.epicgames.net/browse/OR-17066
TAutoConsoleVariable<int32> CVarAnimStateMachineRelevancyReset(TEXT("a.AnimNode.StateMachine.EnableRelevancyReset"), 1, TEXT("Reset State Machine when it becomes relevant"));


void FAnimNode_StateMachine::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	Context.AnimInstanceProxy->RecordMachineWeight(StateMachineIndexInClass, Context.GetFinalBlendWeight());

	// If we just became relevant and haven't been initialized yet, then reinitialize state machine.
	if (!bFirstUpdate && bReinitializeOnBecomingRelevant && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter()) && (CVarAnimStateMachineRelevancyReset.GetValueOnAnyThread() == 1))
	{
		FAnimationInitializeContext InitializationContext(Context.AnimInstanceProxy, Context.SharedContext);
		Initialize_AnyThread(InitializationContext);
	}
	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	// Remove expired transition requests
#if WITH_EDITORONLY_DATA
	HandledTransitionEvents.Reset();
#endif
	for (int32 RequestIndex = QueuedTransitionEvents.Num() - 1; RequestIndex >= 0; --RequestIndex)
	{
		if (QueuedTransitionEvents[RequestIndex].HasExpired())
		{
			UE_LOGF(LogAnimTransitionRequests, Verbose, "'%ls' expired (Machine: %ls)", *QueuedTransitionEvents[RequestIndex].EventName.ToString(), *GetMachineDescription()->MachineName.ToString());
			QueuedTransitionEvents.RemoveAt(RequestIndex, EAllowShrinking::No);
		}
	}
	QueuedTransitionEvents.Shrink();

	const FBakedAnimationStateMachine* Machine = GetMachineDescription();
	if (Machine != nullptr)
	{
		if (Machine->States.Num() == 0)
		{
			return;
		}
		else if(!Machine->States.IsValidIndex(CurrentState))
		{
			// Attempting to catch a crash where the state machine has been freed.
			// Reported as a symptom of a crash in UE-24732 for 4.10. This log message should not appear given changes to
			// re-instancing in 4.11 (see CL 2823202). If it does appear we need to spot integrate CL 2823202 (and supporting 
			// anim re-init changes, probably 2799786 & 2801372).
			UE_LOG(LogAnimation, Warning, TEXT("FAnimNode_StateMachine::Update - Invalid current state, please report. Attempting to use state %d of %d in state machine %d (ptr 0x%" UPTRINT_x_FMT ")"), CurrentState, Machine->States.Num(), StateMachineIndexInClass, Machine);
			UE_LOGF(LogAnimation, Warning, "\t\tWhen updating AnimInstance: %ls", *Context.AnimInstanceProxy->GetAnimInstanceObject()->GetName())

			return;
		}
	}
	else
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_AnimStateMachineUpdate);
#if	STATS
	// Record name of state machine we are updating
	FScopeCycleCounter MachineNameCycleCounter(Machine->GetStatID());
#endif // STATS

#if WITH_EDITOR
	if (DebugForcedStateIndex != INDEX_NONE)
	{
		if (PRIVATE_MachineDescription->States.IsValidIndex(DebugForcedStateIndex))
		{
			SetState(Context, DebugForcedStateIndex, true);
		}
		DebugForcedStateIndex = INDEX_NONE;
	}
#endif

	// On the first update, call the initial state's Start Notify.
	if (bFirstUpdate)
	{
		//Handle enter notify for "first" (after initial transitions) state
		Context.AnimInstanceProxy->AddAnimNotifyFromGeneratedClass(GetStateInfo().StartNotify);
	}

	bool bFoundValidTransition = false;
	int32 TransitionCountThisFrame = 0;

	// Look for legal transitions to take; can move across multiple states in one frame (up to MaxTransitionsPerFrame)
	do
	{
		bFoundValidTransition = false;
		FAnimationPotentialTransition PotentialTransition;
		
		{
			SCOPE_CYCLE_COUNTER(STAT_AnimStateMachineFindTransition);

			// Evaluate possible transitions out of this state
			//@TODO: Evaluate if a set is better than an array for the probably low N encountered here
			TArray<int32, TInlineAllocator<4>> VisitedStateIndices;
			FindValidTransition(Context, GetStateInfo(), /*Out*/ PotentialTransition, /*Out*/ VisitedStateIndices);
		}

		auto CanTransition = [this, &PotentialTransition]()
		{
			if (!PotentialTransition.IsValid())
			{
				return false;
			}

			const FAnimationTransitionBetweenStates& ReferenceTransition = GetTransitionInfo(PotentialTransition.TransitionRule->TransitionIndex);

			// If transition is valid and not waiting on other conditions
			// and we're not doing a transition to self, unless the self-transition can inertialize
			if (PotentialTransition.TargetState != CurrentState || ReferenceTransition.bAllowInertializationForSelfTransitions)
			{
				return true;
			}

			return false;
		};

		if (CanTransition())
		{
			bFoundValidTransition = true;

			const FAnimationTransitionBetweenStates& ReferenceTransition = GetTransitionInfo(PotentialTransition.TransitionRule->TransitionIndex);
			TransitionToState(Context, ReferenceTransition, &PotentialTransition);

			TransitionCountThisFrame++;
		}
	}
	while (bFoundValidTransition && (TransitionCountThisFrame < MaxTransitionsPerFrame));
	
	if (bFirstUpdate)
	{
		if (bSkipFirstUpdateTransition)
		{
			// in the first update, we don't like to transition from entry state
			// so we throw out any transition data at the first update
			ActiveTransitionArray.Reset();
		}
		bFirstUpdate = false;
	}

	StatesUpdated.Reset();

	bool bLastActiveTransitionRequestedInertialization = false;
	
	// Tick the individual state/states that are active
	if (ActiveTransitionArray.Num() > 0)
	{
		// Keep track of states that have blended out to avoid recalling their anim node state functions.
		TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<4>> BlendedOutStates;
		TArray<bool, TInlineAllocator<4>> FinishedTransitions;
		FinishedTransitions.Init(false, ActiveTransitionArray.Num());

		// Update the transitions prior to the states to ensure all transition alphas are correct
		for (int32 Index = 0; Index < ActiveTransitionArray.Num(); ++Index)
		{
			ActiveTransitionArray[Index].Update(Context, CurrentState, /*out*/ FinishedTransitions[Index]);
		}
		
		for (int32 Index = 0; Index < ActiveTransitionArray.Num(); ++Index)
		{
			// The custom graph will tick the needed states
			if (FinishedTransitions[Index])
			{
				// Trigger "Fully Blended Out" of any states that have any weight to them.
				{
					const int32 PreviousStateIndex = ActiveTransitionArray[Index].PreviousState;
					const bool bAttemptingToTransitionOutOfCurrentState = PreviousStateIndex == CurrentState;
					
					if (!bAttemptingToTransitionOutOfCurrentState && StatePoseLinks.IsValidIndex(PreviousStateIndex) && StatePoseLinks[PreviousStateIndex].LinkID != INDEX_NONE)
					{
						if (!BlendedOutStates.Contains(PreviousStateIndex) && FMath::IsNearlyZero(GetStateWeight(PreviousStateIndex)))
						{
							UE::Anim::FNodeFunctionCaller::CallFunction(static_cast<FAnimNode_StateResult*>(StatePoseLinks[PreviousStateIndex].GetLinkNode())->GetStateFullyBlendedOutFunction(), Context, *this);
							BlendedOutStates.Add(PreviousStateIndex);
						}
					}
				}
				
				// only play these events if it is the last transition (most recent, going to current state)
				if (Index == (ActiveTransitionArray.Num() - 1))
				{
					Context.AnimInstanceProxy->AddAnimNotifyFromGeneratedClass(ActiveTransitionArray[Index].EndNotify);
					Context.AnimInstanceProxy->AddAnimNotifyFromGeneratedClass(GetStateInfo().FullyBlendedNotify);

					// Handle events/calls during "Fully Blended In".
					{
						const int32 NextStateIndex = ActiveTransitionArray[Index].NextState;
						
						if (StatePoseLinks.IsValidIndex(NextStateIndex))
						{
							// If our most recent state has fully blended in that means that all our previous states must also blend out and their transitions will be forced
							// to complete, (removed from active transition array), therefore we need to call their fully blended out functions if they haven't been called already. 
							if (ActiveTransitionArray.Num() > 1)
							{
								for (int32 BlendOutIndex = 0; BlendOutIndex < Index; ++BlendOutIndex)
								{
									const int32 PreviousStateIndex = ActiveTransitionArray[BlendOutIndex].PreviousState;
									const bool bAttemptingToTransitionOutOfCurrentState = PreviousStateIndex == CurrentState;

									if (!bAttemptingToTransitionOutOfCurrentState && StatePoseLinks.IsValidIndex(PreviousStateIndex) && StatePoseLinks[PreviousStateIndex].LinkID != INDEX_NONE && !BlendedOutStates.Contains(PreviousStateIndex))
									{
										UE::Anim::FNodeFunctionCaller::CallFunction(static_cast<FAnimNode_StateResult*>(StatePoseLinks[PreviousStateIndex].GetLinkNode())->GetStateFullyBlendedOutFunction(), Context, *this);
										BlendedOutStates.Add(PreviousStateIndex);
									}
								}
							}

							// Now that all possible state fully blended out functions have been called, perform state fully blended in call.
							if (StatePoseLinks.IsValidIndex(NextStateIndex) && StatePoseLinks[NextStateIndex].LinkID != INDEX_NONE)
							{
								UE::Anim::FNodeFunctionCaller::CallFunction(static_cast<FAnimNode_StateResult*>(StatePoseLinks[NextStateIndex].GetLinkNode())->GetStateFullyBlendedInFunction(), Context, *this);
							}
						}
					}
				}

				// we were the last active transition and used inertialization
				if (ActiveTransitionArray[Index].LogicType == ETransitionLogicType::TLT_Inertialization && (ActiveTransitionArray.Num() - 1 ==  Index))
				{
					bLastActiveTransitionRequestedInertialization = true;
				}
			}
			else
			{
				// transition is still active, so tick the required states
				UpdateTransitionStates(Context, ActiveTransitionArray[Index]);
			}
		}
		
		// remove finished transitions here, newer transitions ending means any older ones must complete as well
		for (int32 Index = (ActiveTransitionArray.Num()-1); Index >= 0; --Index)
		{
			// if we find an inactive one, remove all older transitions and break out
			if (!ActiveTransitionArray[Index].bActive)
			{
				ActiveTransitionArray.RemoveAt(0, Index+1, EAllowShrinking::No);
				break;
			}
		}
	}

	//@TODO: StatesUpdated.Contains is a linear search
	// Update the only active state if there are no transitions still in flight
	if (ActiveTransitionArray.Num() == 0 && !IsAConduitState(CurrentState) && !StatesUpdated.Contains(CurrentState))
	{
		UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimInertializationSyncScope> InertializationSync(bLastActiveTransitionRequestedInertialization, Context);
		UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FActiveStateMachineScope> Message(bCreateNotifyMetaData, Context, Context, this, CurrentState);
		
		StatePoseLinks[CurrentState].Update(Context);
	}

	ElapsedTime += Context.GetDeltaTime();

	// Record state weights after transitions/updates are completed
	for (int32 StateIndex = 0; StateIndex < StatePoseLinks.Num(); ++StateIndex)
	{
		const float StateWeight = GetStateWeight(StateIndex);
		if(StateWeight > 0.0f)
		{
			const float CurrentStateElapsedTime = StateIndex == CurrentState ? GetCurrentStateElapsedTime() : 0.0f;
			Context.AnimInstanceProxy->RecordStateWeight(StateMachineIndexInClass, StateIndex, StateWeight, CurrentStateElapsedTime);

			TRACE_ANIM_STATE_MACHINE_STATE(Context, StateMachineIndexInClass, StateIndex, StateWeight, CurrentStateElapsedTime);
		}
	}

#if ANIM_TRACE_ENABLED
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), GetMachineDescription()->MachineName);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Current State"), GetStateInfo().StateName);
	TRACE_ANIM_NODE_VALUE(
		Context,
		TEXT("Queued Transition Requests"), 
		*FString::JoinBy(QueuedTransitionEvents, TEXT(",\n"), [](const FTransitionEvent& TransitionEvent) { return TransitionEvent.ToDebugString(); })
	);
#if WITH_EDITORONLY_DATA
	TRACE_ANIM_NODE_VALUE(
		Context,
		TEXT("Consumed Transition Requests"),
		*FString::JoinBy(HandledTransitionEvents, TEXT(",\n"), [](const FTransitionEvent& TransitionEvent) { return TransitionEvent.EventName.ToString(); })
	);
#endif //WITH_EDITORONLY_DATA
#endif //ANIM_TRACE_ENABLED
}

float FAnimNode_StateMachine::GetRelevantAnimTimeRemaining(const FAnimInstanceProxy* InAnimInstanceProxy, int32 StateIndex) const
{
	if (const FAnimNode_AssetPlayerRelevancyBase* AssetPlayer = GetRelevantAssetPlayerInterfaceFromState(InAnimInstanceProxy, GetStateInfo(StateIndex)))
	{
		if (AssetPlayer->GetAnimAsset())
		{
			return AssetPlayer->GetCurrentAssetLength() - AssetPlayer->GetCurrentAssetTimePlayRateAdjusted();
		}
	}

	return MAX_flt;
}

float FAnimNode_StateMachine::GetRelevantAnimTimeRemainingFraction(const FAnimInstanceProxy* InAnimInstanceProxy, int32 StateIndex) const
{
	if (const FAnimNode_AssetPlayerRelevancyBase* AssetPlayer = GetRelevantAssetPlayerInterfaceFromState(InAnimInstanceProxy, GetStateInfo(StateIndex)))
	{
		if (AssetPlayer->GetAnimAsset())
		{
			float Length = AssetPlayer->GetCurrentAssetLength();
			if (Length > 0.0f)
			{
				return (Length - AssetPlayer->GetCurrentAssetTimePlayRateAdjusted()) / Length;
			}
		}
	}

	return 1.f;
}

const FAnimNode_AssetPlayerRelevancyBase* FAnimNode_StateMachine::GetRelevantAssetPlayerInterfaceFromState(const FAnimInstanceProxy* InAnimInstanceProxy, const FBakedAnimationState& StateInfo) const
{
	const FAnimNode_AssetPlayerRelevancyBase* ResultPlayer = nullptr;
	float MaxWeight = 0.0f;

	auto EvaluatePlayerWeight = [&MaxWeight, &ResultPlayer](const FAnimNode_AssetPlayerRelevancyBase* Player)
	{
		if (!Player->GetIgnoreForRelevancyTest() && Player->GetCachedBlendWeight() > MaxWeight)
		{
			MaxWeight = Player->GetCachedBlendWeight();
			ResultPlayer = Player;
		}
	};

	for (const int32& PlayerIdx : StateInfo.PlayerNodeIndices)
	{
		if (const FAnimNode_AssetPlayerRelevancyBase* Player = InAnimInstanceProxy->GetNodeFromIndex<FAnimNode_AssetPlayerRelevancyBase>(PlayerIdx))
		{
			EvaluatePlayerWeight(Player);
		}
	}

	// Get all layer node indices that are part of this state
	for (const int32& LinkedAnimNodeIdx : StateInfo.LayerNodeIndices)
	{
		// Try and retrieve the actual node object
		if (const FAnimNode_LinkedAnimGraph* LinkedAnimGraph = InAnimInstanceProxy->GetNodeFromIndex<FAnimNode_LinkedAnimGraph>(LinkedAnimNodeIdx))
		{
			// Retrieve the AnimInstance running for this linked anim graph/layer
			if (const UAnimInstance* CurrentTarget = LinkedAnimGraph->GetTargetInstance<UAnimInstance>())
			{
				FName GraphName;
				if (const FAnimNode_LinkedAnimLayer* LinkedAnimLayer = InAnimInstanceProxy->GetNodeFromIndex<FAnimNode_LinkedAnimLayer>(LinkedAnimNodeIdx))
				{
					GraphName = LinkedAnimLayer->Layer;
				}
				else
				{
					GraphName = DefaultAnimGraphName;
				}

				// Retrieve all asset player nodes from the corresponding Anim blueprint class and apply same logic to find highest weighted asset player 
				TArray<const FAnimNode_AssetPlayerRelevancyBase*> PlayerNodesInLayer = CurrentTarget->GetInstanceRelevantAssetPlayers(GraphName);
				for (const FAnimNode_AssetPlayerRelevancyBase* Player : PlayerNodesInLayer)
				{
					EvaluatePlayerWeight(Player);
				}
			}
		}
	}
	return ResultPlayer;
}

bool FAnimNode_StateMachine::FindValidTransition(const FAnimationUpdateContext& Context, const FBakedAnimationState& StateInfo, /*out*/ FAnimationPotentialTransition& OutPotentialTransition, /*out*/ TArray<int32, TInlineAllocator<4>>& OutVisitedStateIndices)
{
	// There is a possibility we'll revisit states connected through conduits,
	// so we can avoid doing unnecessary work (and infinite loops) by caching off states we have already checked
	const int32 CheckingStateIndex = GetStateIndex(StateInfo);
	if (OutVisitedStateIndices.Contains(CheckingStateIndex))
	{
		return false;
	}
	OutVisitedStateIndices.Add(CheckingStateIndex);

	const IAnimClassInterface* AnimBlueprintClass = Context.GetAnimClass();

	// Conduit 'states' have an additional entry rule which must be true to consider taking any transitions via the conduit
	//@TODO: It would add flexibility to be able to define this on normal state nodes as well, assuming the dual-graph editing is sorted out
	if (FAnimNode_TransitionResult* StateEntryRuleNode = GetNodeFromPropertyIndex<FAnimNode_TransitionResult>(Context.AnimInstanceProxy->GetAnimInstanceObject(), AnimBlueprintClass, StateInfo.EntryRuleNodeIndex))
	{
		if (StateEntryRuleNode->NativeTransitionDelegate.IsBound())
		{
			// attempt to evaluate native rule
			StateEntryRuleNode->bCanEnterTransition = StateEntryRuleNode->NativeTransitionDelegate.Execute();
		}
		else
		{
			// Execute it and see if we can take this rule
			StateEntryRuleNode->GetEvaluateGraphExposedInputs().Execute(Context);
		}

		// not ok, back out
		if (!StateEntryRuleNode->bCanEnterTransition)
		{
			return false;
		}
	}

	const double WorldTime = Context.AnimInstanceProxy->GetAnimInstanceObject()->GetWorld()->GetTimeSeconds();

	const int32 NumTransitions = StateInfo.Transitions.Num();
	for (int32 TransitionIndex = 0; TransitionIndex < NumTransitions; ++TransitionIndex)
	{
		const FBakedStateExitTransition& TransitionRule = StateInfo.Transitions[TransitionIndex];
		if (TransitionRule.CanTakeDelegateIndex == INDEX_NONE)
		{
			continue;
		}

		// Per-transition opt-in: skip rule evaluation entirely when the state machine's update context is inactive.
		if (TransitionRule.bOnlyEvaluateWhenActive && !Context.IsActive())
		{
			continue;
		}

		const FAnimationTransitionBetweenStates& TransitionInfo = GetTransitionInfo(TransitionRule.TransitionIndex);
		const int32 NextState = TransitionInfo.NextState;
		const FBakedAnimationState& NextStateInfo = GetStateInfo(NextState);

		// Check transition de-bounce time
		if (TransitionInfo.MinTimeBeforeReentry >= 0.0f && (WorldTime - LastStateEntryTime[NextState]) < TransitionInfo.MinTimeBeforeReentry)
		{
			continue;
		}

		FAnimNode_TransitionResult* ResultNode = GetNodeFromPropertyIndex<FAnimNode_TransitionResult>(Context.AnimInstanceProxy->GetAnimInstanceObject(), AnimBlueprintClass, TransitionRule.CanTakeDelegateIndex);
		float CrossfadeTimeAdjustment = 0.f;

		// If we require a valid sync group, check that first.
		if ((TransitionRule.SyncGroupNameToRequireValidMarkersRule != NAME_None)
			&& !Context.AnimInstanceProxy->IsSyncGroupValid(TransitionRule.SyncGroupNameToRequireValidMarkersRule))
		{
			ResultNode->bCanEnterTransition = false;
		}
		else if (ResultNode->NativeTransitionDelegate.IsBound())
		{
			// attempt to evaluate native rule
			ResultNode->bCanEnterTransition = ResultNode->NativeTransitionDelegate.Execute();
		}
		else if (TransitionRule.bAutomaticRemainingTimeRule)
		{
			bool bCanEnterTransition = false;
			if (const FAnimNode_AssetPlayerRelevancyBase* RelevantPlayer = GetRelevantAssetPlayerInterfaceFromState(Context, StateInfo))
			{
				if (const UAnimationAsset* AnimAsset = RelevantPlayer->GetAnimAsset())
				{
					float AnimTimeRemaining = AnimAsset->GetPlayLength() - RelevantPlayer->GetAccumulatedTime();

					// Check whether the player looped last frame
					if (RelevantPlayer->IsLooping() && AnimTimeRemaining > 0.f)
					{
						if (const FDeltaTimeRecord* DeltaTimeRecord = RelevantPlayer->GetDeltaTimeRecord())
						{
							if (DeltaTimeRecord->IsPreviousValid())
							{
								const float DeltaTimeFromPlayer = RelevantPlayer->GetAccumulatedTime() - DeltaTimeRecord->GetPrevious();
								const float DeltaTimeFromRecord = DeltaTimeRecord->Delta;

								// deal with negative play rates
								if (DeltaTimeFromPlayer * DeltaTimeFromRecord < 0.f)
								{
									AnimTimeRemaining = 0.f;
								}
							}
						}
					}

					// For transitions that go to a conduit the user is not able to edit the transition's cross fade duration,
					// therefore we force the cross fade duration to zero to ensure the transition is always triggerred upon reaching the end of the animation.
					const float CrossfadeDuration = GetStateInfo(TransitionInfo.NextState).bIsAConduit ? 0.0f : TransitionInfo.CrossfadeDuration;

					// Allow for used to determine the automatic transition trigger time or fallback to using cross fade duration.
					const float TransitionTriggerTime = (TransitionRule.AutomaticRuleTriggerTime >= 0.0f) ? TransitionRule.AutomaticRuleTriggerTime : CrossfadeDuration;
					CrossfadeTimeAdjustment = TransitionTriggerTime - AnimTimeRemaining;

					// Trigger transition only if we have "CrossfadeTimeAdjustment" seconds left before reaching animation end boundary.
					bCanEnterTransition = (CrossfadeTimeAdjustment >= 0.f);
				}
			}
			
			ResultNode->bCanEnterTransition = bCanEnterTransition;
		}			
		else 
		{
			// Execute it and see if we can take this rule
			ResultNode->GetEvaluateGraphExposedInputs().Execute(Context);
		}

		if (ResultNode->bCanEnterTransition == TransitionRule.bDesiredTransitionReturnValue)
		{
			// if next state is a conduit we want to check for transitions using that state as the root
			if (NextStateInfo.bIsAConduit)
			{
				if (FindValidTransition(Context, NextStateInfo, /*out*/ OutPotentialTransition, /*out*/ OutVisitedStateIndices))
				{
					OutPotentialTransition.SourceTransitionIndices.Add(TransitionRule.TransitionIndex);

					return true;
				}					
			}
			// otherwise we have found a content state, so we can record our potential transition
			else
			{
				// clear out any potential transition we already have
				OutPotentialTransition.Clear();

				// fill out the potential transition information
				OutPotentialTransition.TransitionRule = &TransitionRule;
				OutPotentialTransition.TargetState = NextState;
				OutPotentialTransition.CrossfadeTimeAdjustment = CrossfadeTimeAdjustment;
				OutPotentialTransition.SourceTransitionIndices.Add(TransitionRule.TransitionIndex);

				return true;
			}
		}
	}

	return false;
}

void FAnimNode_StateMachine::UpdateTransitionStates(const FAnimationUpdateContext& Context, FAnimationActiveTransitionEntry& Transition)
{
	if (Transition.bActive)
	{
		switch (Transition.LogicType)
		{
		case ETransitionLogicType::TLT_StandardBlend:
			{
				// update both states
				{
					FAnimationUpdateContext StateContext = Context.FractionalWeight(GetStateWeight(Transition.PreviousState));
					UpdateState(Transition.PreviousState, Transition.PreviousState == CurrentState ? StateContext : StateContext.AsInactive());
				}
				{
					FAnimationUpdateContext StateContext = Context.FractionalWeight(GetStateWeight(Transition.NextState));
					UpdateState(Transition.NextState, (Transition.NextState == CurrentState) ? StateContext : StateContext.AsInactive());
				}
			}
			break;

		case ETransitionLogicType::TLT_Inertialization:
			{
				UE::Anim::TScopedGraphMessage<UE::Anim::FAnimInertializationSyncScope> InertializationSync(Context);

				// update target state
				UpdateState(Transition.NextState, (Transition.NextState == CurrentState) ? Context : Context.AsInactive());
			}
			break;

		case ETransitionLogicType::TLT_Custom:
			{
				if (Transition.CustomTransitionGraph.LinkID != INDEX_NONE)
				{
					Transition.CustomTransitionGraph.Update(Context);

					for (TArray<FAnimNode_TransitionPoseEvaluator*>::TIterator PoseEvaluatorListIt = Transition.PoseEvaluators.CreateIterator(); PoseEvaluatorListIt; ++PoseEvaluatorListIt)
					{
						FAnimNode_TransitionPoseEvaluator* Evaluator = *PoseEvaluatorListIt;
						if (Evaluator->InputNodeNeedsUpdate(Context))
						{
							const bool bUsePreviousState = (Evaluator->DataSource == EEvaluatorDataSource::EDS_SourcePose);
							const int32 EffectiveStateIndex = bUsePreviousState ? Transition.PreviousState : Transition.NextState;
							FAnimationUpdateContext ContextToUse = Context.FractionalWeight(bUsePreviousState ? (1.0f - Transition.Alpha) : Transition.Alpha);
							UpdateState(EffectiveStateIndex, (EffectiveStateIndex == CurrentState) ? ContextToUse : ContextToUse.AsInactive());
						}
					}
				}
			}
			break;

		default:
			break;
		}
	}
}

void FAnimNode_StateMachine::Evaluate_AnyThread(FPoseContext& Output)
{
	if (const FBakedAnimationStateMachine* Machine = GetMachineDescription())
	{
		const bool bCurrentStateIsConduit = IsAConduitState(CurrentState);
#if WITH_EDITORONLY_DATA
		if (bCurrentStateIsConduit)
		{ 
			UAnimBlueprint* AnimBlueprint = Output.AnimInstanceProxy->GetAnimBlueprint();

			FText Message = FText::Format(LOCTEXT("CurrentStateIsConduitWarning", "Current state is a conduit and will reset to ref pose. This can happen if a conduit is an entry state and there isn't at least one valid transition to take. State Machine({0}), Conduit({1}), AnimBlueprint({2})."),
				FText::FromName(Machine->MachineName), FText::FromName(GetCurrentStateName()), FText::FromString(GetPathNameSafe(AnimBlueprint)));
			Output.LogMessage(EMessageSeverity::Error, Message);
		}
#else
		ensureMsgf(!bCurrentStateIsConduit, TEXT("Current state is a conduit and will reset to ref pose. State Machine(%s), Conduit(%s), AnimInstance(%s)."),
			*Machine->MachineName.ToString(), *GetCurrentStateName().ToString(), *Output.AnimInstanceProxy->GetAnimInstanceName());
#endif

		if (bCurrentStateIsConduit || Machine->States.Num() == 0 || !Machine->States.IsValidIndex(CurrentState))
		{
			Output.Pose.ResetToRefPose();
			return;
		}
	}
	else
	{
		Output.Pose.ResetToRefPose();
		return;
	}

	ANIM_MT_SCOPE_CYCLE_COUNTER(EvaluateAnimStateMachine, !IsInGameThread());
	
	if (ActiveTransitionArray.Num() > 0)
	{
		check(Output.AnimInstanceProxy->GetSkeleton());
		
		check(StateCachedPoses.Num() == 0);
		StateCachedPoses.AddZeroed(StatePoseLinks.Num());

		//each transition stomps over the last because they will already include the output from the transition before it
		for (int32 Index = 0; Index < ActiveTransitionArray.Num(); ++Index)
		{
			// if there is any source pose, blend it here
			FAnimationActiveTransitionEntry& ActiveTransition = ActiveTransitionArray[Index];
			
			// when evaluating multiple transitions we need to store the pose from previous results
			// so we can feed the next transitions
			const bool bIntermediatePoseIsValid = Index > 0;

			if (ActiveTransition.bActive)
			{
				switch (ActiveTransition.LogicType)
				{
				case ETransitionLogicType::TLT_StandardBlend:
					EvaluateTransitionStandardBlend(Output, ActiveTransition, bIntermediatePoseIsValid);
					break;
				case ETransitionLogicType::TLT_Inertialization:
					EvaluateState(ActiveTransition.NextState, Output);
					break;
				case ETransitionLogicType::TLT_Custom:
					EvaluateTransitionCustomBlend(Output, ActiveTransition, bIntermediatePoseIsValid);
					break;
				default:
					break;
				}
			}
		}

		// Ensure that all of the resulting rotations are normalized
		Output.Pose.NormalizeRotations();

		// Clear our cache
		for (auto CachedPose : StateCachedPoses)
		{
			delete CachedPose;
		}
		StateCachedPoses.Reset();
	}
	else if (!IsAConduitState(CurrentState))
	{
		// Make sure CacheBones has been called before evaluating.
		ConditionallyCacheBonesForState(CurrentState, Output);

		// Evaluate the current state
		StatePoseLinks[CurrentState].Evaluate(Output);
	}
}


void FAnimNode_StateMachine::EvaluateTransitionStandardBlend(FPoseContext& Output, FAnimationActiveTransitionEntry& Transition, bool bIntermediatePoseIsValid)
{
	if (bIntermediatePoseIsValid)
	{
		FPoseContext PreviousStateResult(Output); 
		PreviousStateResult = Output;
		const FPoseContext& NextStateResult = EvaluateState(Transition.NextState, Output);
		EvaluateTransitionStandardBlendInternal(Output, Transition, PreviousStateResult, NextStateResult);
	}
	else
	{
		const FPoseContext& PreviousStateResult = EvaluateState(Transition.PreviousState, Output);
		const FPoseContext& NextStateResult = EvaluateState(Transition.NextState, Output);
		EvaluateTransitionStandardBlendInternal(Output, Transition, PreviousStateResult, NextStateResult);
	}
}

void FAnimNode_StateMachine::EvaluateTransitionStandardBlendInternal(FPoseContext& Output, FAnimationActiveTransitionEntry& Transition, const FPoseContext& PreviousStateResult, const FPoseContext& NextStateResult)
{	
	// Blend it in
	const ScalarRegister VPreviousWeight(1.0f - Transition.Alpha);
	const ScalarRegister VWeight(Transition.Alpha);

	// If we have a blend profile we need to blend per bone.
	if (Transition.BlendProfile)
	{
		const FBoneContainer& RequiredBones = Output.AnimInstanceProxy->GetRequiredBones();
		TSharedPtr<IInterpolationIndexProvider::FPerBoneInterpolationData> Data = Transition.BlendProfile->GetPerBoneInterpolationData(Output.AnimInstanceProxy->GetSkeleton());

		// If we have some skeleton remapping and the source data comes from another skeleton.
		// This is a slightly slower path, so we made two branches, one with remapping and one without.
		const FSkeletonRemapping& Remapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(Transition.BlendProfile->GetSkeleton(), Output.AnimInstanceProxy->GetSkeleton());
		if (Remapping.IsValid())
		{
			for (const FCompactPoseBoneIndex TargetBoneIndex : Output.Pose.ForEachBoneIndex())
			{
				const FSkeletonPoseBoneIndex SourceSkelBoneIndex(Remapping.GetSourceSkeletonBoneIndex(TargetBoneIndex.GetInt()));
				const FCompactPoseBoneIndex SourceBoneIndex = FCompactPoseBoneIndex(RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(SourceSkelBoneIndex));
				const int32 PerBoneIndex = (SourceBoneIndex != INDEX_NONE) ? Transition.BlendProfile->GetPerBoneInterpolationIndex(SourceBoneIndex, RequiredBones, Data.Get()) : INDEX_NONE;

				// Use defined per-bone scale if the bone has a scale specified in the blend profile.
				const ScalarRegister FirstWeight = (PerBoneIndex != INDEX_NONE) ? ScalarRegister(Transition.StateBlendData[1].PerBoneBlendData[PerBoneIndex]) : ScalarRegister(VPreviousWeight);
				const ScalarRegister SecondWeight = (PerBoneIndex != INDEX_NONE) ? ScalarRegister(Transition.StateBlendData[0].PerBoneBlendData[PerBoneIndex]) : ScalarRegister(VWeight);
				Output.Pose[TargetBoneIndex] = PreviousStateResult.Pose[TargetBoneIndex] * FirstWeight;
				Output.Pose[TargetBoneIndex].AccumulateWithShortestRotation(NextStateResult.Pose[TargetBoneIndex], SecondWeight);
			}
		}
		else // There is no skeleton remapping or we are using the same skeleton as the source.
		{
			for (const FCompactPoseBoneIndex TargetBoneIndex : Output.Pose.ForEachBoneIndex())
			{
				const int32 PerBoneIndex = Transition.BlendProfile->GetPerBoneInterpolationIndex(TargetBoneIndex, RequiredBones, Data.Get());

				// Use defined per-bone scale if the bone has a scale specified in the blend profile.
				const ScalarRegister FirstWeight = (PerBoneIndex != INDEX_NONE) ? ScalarRegister(Transition.StateBlendData[1].PerBoneBlendData[PerBoneIndex]) : ScalarRegister(VPreviousWeight);
				const ScalarRegister SecondWeight = (PerBoneIndex != INDEX_NONE) ? ScalarRegister(Transition.StateBlendData[0].PerBoneBlendData[PerBoneIndex]) : ScalarRegister(VWeight);
				Output.Pose[TargetBoneIndex] = PreviousStateResult.Pose[TargetBoneIndex] * FirstWeight;
				Output.Pose[TargetBoneIndex].AccumulateWithShortestRotation(NextStateResult.Pose[TargetBoneIndex], SecondWeight);
			}
		}
	}
	else
	{
		for(FCompactPoseBoneIndex BoneIndex : Output.Pose.ForEachBoneIndex())
		{
			Output.Pose[BoneIndex] = PreviousStateResult.Pose[BoneIndex] * VPreviousWeight;
			Output.Pose[BoneIndex].AccumulateWithShortestRotation(NextStateResult.Pose[BoneIndex], VWeight);
		}
	}
	
	// blend curve in
	Output.Curve.Override(PreviousStateResult.Curve, 1.0 - Transition.Alpha);
	Output.Curve.Accumulate(NextStateResult.Curve, Transition.Alpha);
		
	// reset attributes container as it can contain a previously handled active transition (and attributes expects blending into an empty container)
	Output.CustomAttributes.Empty();
	UE::Anim::Attributes::BlendAttributes({ PreviousStateResult.CustomAttributes, NextStateResult.CustomAttributes }, { 1.0f - Transition.Alpha, Transition.Alpha }, { 0, 1 }, Output.CustomAttributes);
}

void FAnimNode_StateMachine::EvaluateTransitionCustomBlend(FPoseContext& Output, FAnimationActiveTransitionEntry& Transition, bool bIntermediatePoseIsValid)
{
	if (Transition.CustomTransitionGraph.LinkID != INDEX_NONE)
	{
		for (TArray<FAnimNode_TransitionPoseEvaluator*>::TIterator PoseEvaluatorListIt(Transition.PoseEvaluators); PoseEvaluatorListIt; ++PoseEvaluatorListIt)
		{
			FAnimNode_TransitionPoseEvaluator* Evaluator = *PoseEvaluatorListIt;
			if (Evaluator->InputNodeNeedsEvaluate())
			{
				// All input evaluators that use the intermediate pose can grab it from the current output.
				const bool bUseIntermediatePose = bIntermediatePoseIsValid && (Evaluator->DataSource == EEvaluatorDataSource::EDS_SourcePose);

				// otherwise we need to evaluate the nodes they reference
				if (!bUseIntermediatePose)
				{
					const bool bUsePreviousState = (Evaluator->DataSource == EEvaluatorDataSource::EDS_SourcePose);
					const int32 EffectiveStateIndex = bUsePreviousState ? Transition.PreviousState : Transition.NextState;
					const FPoseContext& PoseEvalResult = EvaluateState(EffectiveStateIndex, Output);

					// push transform to node.
					Evaluator->CachePose(PoseEvalResult);
				}
				else
				{
					// push transform to node.
					Evaluator->CachePose(Output);
				}
			}
		}

		FPoseContext StatePoseResult(Output);
		Transition.CustomTransitionGraph.Evaluate(StatePoseResult);

		// First pose will just overwrite the destination
		for (const FCompactPoseBoneIndex BoneIndex : Output.Pose.ForEachBoneIndex())
		{
			Output.Pose[BoneIndex] = StatePoseResult.Pose[BoneIndex];
		}

		// Copy curve over also, replacing current.
		Output.Curve.CopyFrom(StatePoseResult.Curve);

		// Copy attributes over, replacing current.
		Output.CustomAttributes.CopyFrom(StatePoseResult.CustomAttributes);
	}
}

void FAnimNode_StateMachine::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(%s->%s)"), *GetMachineDescription()->MachineName.ToString(), *GetStateInfo().StateName.ToString());

	DebugData.AddDebugItem(DebugLine);
	for (int32 PoseIndex = 0; PoseIndex < StatePoseLinks.Num(); ++PoseIndex)
	{
		FString StateName = FString::Printf(TEXT("(State: %s)"), *GetStateInfo(PoseIndex).StateName.ToString());
		StatePoseLinks[PoseIndex].GatherDebugData(DebugData.BranchFlow(GetStateWeight(PoseIndex), StateName));
	}
}

void FAnimNode_StateMachine::SetStateInternal(int32 NewStateIndex)
{
	checkSlow(PRIVATE_MachineDescription);
	ensure(!IsAConduitState(NewStateIndex) || bAllowConduitEntryStates);
	CurrentState = FMath::Clamp<int32>(NewStateIndex, 0, PRIVATE_MachineDescription->States.Num() - 1);
	check(CurrentState == NewStateIndex);
	ElapsedTime = 0.0f;
}

void FAnimNode_StateMachine::SetState(const FAnimationBaseContext& Context, int32 NewStateIndex, bool bAllowReEntry)
{
	SCOPE_CYCLE_COUNTER(Stat_StateMachineSetState);

	if (NewStateIndex != CurrentState || bAllowReEntry)
	{
		const int32 PrevStateIndex = CurrentState;
		if(CurrentState != INDEX_NONE && CurrentState < OnGraphStatesExited.Num())
		{
			OnGraphStatesExited[CurrentState].ExecuteIfBound(*this, CurrentState, NewStateIndex);
		}

		if (StatePoseLinks.IsValidIndex(CurrentState) && StatePoseLinks[CurrentState].LinkID != INDEX_NONE)
		{
			UE::Anim::FNodeFunctionCaller::CallFunction(static_cast<FAnimNode_StateResult*>(StatePoseLinks[CurrentState].GetLinkNode())->GetStateExitFunction(), Context, *this);
		}

		bool bForceReset = false;

		if(PRIVATE_MachineDescription->States.IsValidIndex(NewStateIndex))
		{
			const FBakedAnimationState& BakedCurrentState = PRIVATE_MachineDescription->States[NewStateIndex];
			bForceReset = BakedCurrentState.bAlwaysResetOnEntry;
		}

		// Determine if the new state is active or not
		const bool bAlreadyActive = GetStateWeight(NewStateIndex) > 0.0f;

		SetStateInternal(NewStateIndex);

		// Record entry time for potential de-bouncing
		LastStateEntryTime[CurrentState] = Context.AnimInstanceProxy->GetAnimInstanceObject()->GetWorld()->GetTimeSeconds();

		// Clear any currently cached blend weights for asset player nodes.
		// This stops any zero length blends holding on to old weights
		for(int32 PlayerIndex : GetStateInfo(CurrentState).PlayerNodeIndices)
		{
			if(FAnimNode_AssetPlayerRelevancyBase* Player = Context.AnimInstanceProxy->GetMutableNodeFromIndex<FAnimNode_AssetPlayerRelevancyBase>(PlayerIndex))
			{
				Player->ClearCachedBlendWeight();
			}
		}

		// Clear any currently cached blend weights for asset player nodes in layers.
		for (const int32& LinkedAnimNodeIdx : GetStateInfo(CurrentState).LayerNodeIndices)
		{
			// Try and retrieve the actual node object
			if (FAnimNode_LinkedAnimGraph* LinkedAnimGraph = Context.AnimInstanceProxy->GetMutableNodeFromIndex<FAnimNode_LinkedAnimGraph>(LinkedAnimNodeIdx))
			{
				// Retrieve the AnimInstance running for this layer
				if (UAnimInstance* CurrentTarget = LinkedAnimGraph->GetTargetInstance<UAnimInstance>())
				{
					FName GraphName;
					if (const FAnimNode_LinkedAnimLayer* LinkedAnimLayer = Context.AnimInstanceProxy->GetNodeFromIndex<FAnimNode_LinkedAnimLayer>(LinkedAnimNodeIdx))
					{
						GraphName = LinkedAnimLayer->Layer;
					}
					else
					{
						GraphName = DefaultAnimGraphName;
					}

					// Retrieve all asset player nodes from the corresponding Anim blueprint class and clear their cached blend weight
					TArray<FAnimNode_AssetPlayerRelevancyBase*> PlayerNodesInLayer = CurrentTarget->GetMutableInstanceRelevantAssetPlayers(GraphName);
					for (FAnimNode_AssetPlayerRelevancyBase* Player : PlayerNodesInLayer)
					{
						Player->ClearCachedBlendWeight();
					}
				}
			}
		}

		if ((!bAlreadyActive || bForceReset) && !IsAConduitState(NewStateIndex))
		{
			// Initialize the new state since it's not part of an active transition (and thus not still initialized)
			FAnimationInitializeContext InitContext(Context.AnimInstanceProxy, Context.SharedContext);
			StatePoseLinks[NewStateIndex].Initialize(InitContext);

			// Also call cache bones if needed
			// Note dont call CacheBones here if we are in the process of whole-graph initialization as a 'never updated' counter
			// will not perform its 'minimal update guard' duty and every call will end up getting though, performing duplicate work
			// over Save/UseCachedPose boundaries etc.
			// This is OK because CacheBones is actually called before updating the graph anyway after whole-graph initialization
			if(Context.AnimInstanceProxy->GetCachedBonesCounter().HasEverBeenUpdated())
			{
				ConditionallyCacheBonesForState(NewStateIndex, Context);
			}
		}

		if (StatePoseLinks.IsValidIndex(CurrentState) && StatePoseLinks[CurrentState].LinkID != INDEX_NONE)
		{
			UE::Anim::FNodeFunctionCaller::CallFunction(static_cast<FAnimNode_StateResult*>(StatePoseLinks[CurrentState].GetLinkNode())->GetStateEntryFunction(), Context, *this);
		}

		if(CurrentState != INDEX_NONE && CurrentState < OnGraphStatesEntered.Num())
		{
			OnGraphStatesEntered[CurrentState].ExecuteIfBound(*this, PrevStateIndex, CurrentState);
		}
	}
}

void FAnimNode_StateMachine::TransitionToState(const FAnimationUpdateContext& Context, const FAnimationTransitionBetweenStates& TransitionInfo, const FAnimationPotentialTransition* BakedTransitionInfo)
{
	// let the latest transition know it has been interrupted
	if ((ActiveTransitionArray.Num() > 0) && ActiveTransitionArray[ActiveTransitionArray.Num() - 1].bActive)
	{
		Context.AnimInstanceProxy->AddAnimNotifyFromGeneratedClass(ActiveTransitionArray[ActiveTransitionArray.Num() - 1].InterruptNotify);
	}

	const int32 PreviousState = CurrentState;
	const int32 NextState = TransitionInfo.NextState;

	// Fire off Notifies for state transition
	Context.AnimInstanceProxy->AddAnimNotifyFromGeneratedClass(GetStateInfo(PreviousState).EndNotify);
	Context.AnimInstanceProxy->AddAnimNotifyFromGeneratedClass(GetStateInfo(NextState).StartNotify);

	FAnimationActiveTransitionEntry* PreviousTransitionForNextState = nullptr;
	for (int32 i = ActiveTransitionArray.Num() - 1; i >= 0; --i)
	{
		FAnimationActiveTransitionEntry& TransitionEntry = ActiveTransitionArray[i];
		if (TransitionEntry.PreviousState == NextState)
		{
			PreviousTransitionForNextState = &TransitionEntry;
			break;
		}
	}

	// Don't add a transition if the previous state is a conduit (likely means we're finding the best entry state)
	if (!IsAConduitState(PreviousState))
	{

		// If we have baked transition rule info available, use it
		float CrossFadeTimeAdjustment = 0.0f;
		const FBakedStateExitTransition* BakedExitTransition = nullptr;
		TArray<int32, TInlineAllocator<3>> SourceTransitionIndices;
		if (BakedTransitionInfo)
		{
			CrossFadeTimeAdjustment = BakedTransitionInfo->CrossfadeTimeAdjustment;
			BakedExitTransition = BakedTransitionInfo->TransitionRule;
			SourceTransitionIndices = BakedTransitionInfo->SourceTransitionIndices;
		}

		// Get the current weight of the next state, which may be non-zero
		const float ExistingWeightOfNextState = GetStateWeight(NextState);

		// Push the transition onto the stack
		FAnimationActiveTransitionEntry& NewTransition = ActiveTransitionArray.Emplace_GetRef(NextState, ExistingWeightOfNextState, PreviousState, TransitionInfo, CrossFadeTimeAdjustment);
		if ((TransitionInfo.LogicType == ETransitionLogicType::TLT_Custom) && BakedExitTransition)
		{
			NewTransition.InitializeCustomGraphLinks(Context, *BakedExitTransition);
		}

		// Initialize blend data if necessary
		if (NewTransition.BlendProfile)
		{
			NewTransition.StateBlendData.AddZeroed(2);
			NewTransition.StateBlendData[0].PerBoneBlendData.AddZeroed(NewTransition.BlendProfile->GetNumBlendEntries());
			NewTransition.StateBlendData[1].PerBoneBlendData.AddZeroed(NewTransition.BlendProfile->GetNumBlendEntries());
		}

		auto ShouldInertialize = [MachineDescription = PRIVATE_MachineDescription, NextState, &TransitionInfo, &ExistingWeightOfNextState]()
		{
			// We are set to inertialize
			if (TransitionInfo.LogicType == ETransitionLogicType::TLT_Inertialization)
			{
				return true;
			}
			// Fall back if we are allowed to and the next state has weight, to hide a pop if the state resets
			else if (TransitionInfo.bAllowInertializationForSelfTransitions && ExistingWeightOfNextState > 0.0f)
			{
				const FBakedAnimationState& BakedCurrentState = MachineDescription->States[NextState];
				// If the target state will not reset, then there's no need to inertialize.
				if (BakedCurrentState.bAlwaysResetOnEntry)
				{
					return true;
				}
			}
			return false;
		};

		if (ShouldInertialize())
		{
			UE::Anim::IInertializationRequester* InertializationRequester = Context.GetMessage<UE::Anim::IInertializationRequester>();
			if (InertializationRequester)
			{
				FInertializationRequest Request;
				Request.Duration = TransitionInfo.CrossfadeDuration;
				Request.BlendProfile = TransitionInfo.BlendProfile;
				Request.bUseBlendMode = true;
				Request.BlendMode = TransitionInfo.BlendMode;
				Request.CustomBlendCurve = TransitionInfo.CustomCurve;
#if ANIM_TRACE_ENABLED
				Request.DescriptionString = FText::Format(LOCTEXT("InertializationRequestDescription", 
					"\"{0}\" Transition from \"{1}\" to \"{2}\""), 
					FText::FromName(GetMachineDescription()->MachineName),
					FText::FromName(GetStateInfo(TransitionInfo.PreviousState).StateName),
					FText::FromName(GetStateInfo(TransitionInfo.NextState).StateName)).ToString();
				Request.NodeId = Context.GetCurrentNodeId();
				Request.AnimInstance = Context.AnimInstanceProxy->GetAnimInstanceObject();
#endif

				InertializationRequester->RequestInertialization(Request);
				InertializationRequester->AddDebugRecord(*Context.AnimInstanceProxy, Context.GetCurrentNodeId());
			}
			else
			{
				LogInertializationRequestError(Context, PreviousState, NextState);
			}
		}

		NewTransition.SourceTransitionIndices = SourceTransitionIndices;

		if (!bFirstUpdate || (bFirstUpdate && !bSkipFirstUpdateTransition))
		{
			Context.AnimInstanceProxy->AddAnimNotifyFromGeneratedClass(NewTransition.StartNotify);
		}
	}

	ConsumeMarkedTransitionEvents();
	SetState(Context, NextState, TransitionInfo.bAllowInertializationForSelfTransitions);
}

float FAnimNode_StateMachine::GetStateWeight(int32 StateIndex) const
{
	const int32 NumTransitions = ActiveTransitionArray.Num();
	if (NumTransitions > 0)
	{
		// Determine the overall weight of the state here.
		float TotalWeight = 0.0f;
		for (int32 Index = 0; Index < NumTransitions; ++Index)
		{
			const FAnimationActiveTransitionEntry& Transition = ActiveTransitionArray[Index];

			float SourceWeight = (1.0f - Transition.Alpha);

			// After the first transition, so source weight is the fraction of how much all previous transitions contribute to the final weight.
			// So if our second transition is 50% complete, and our target state was 80% of the first transition, then that number will be multiplied by this weight
			if (Index > 0)
			{
				TotalWeight *= SourceWeight;
			}
			//during the first transition the source weight represents the actual state weight
			else if (Transition.PreviousState == StateIndex)
			{
				TotalWeight += SourceWeight;
			}

			// The next state weight is the alpha of this transition. We always just add the value, it will be reduced down if there are any newer transitions
			if (Transition.NextState == StateIndex)
			{
				TotalWeight += Transition.Alpha;
			}

		}

		return FMath::Clamp<float>(TotalWeight, 0.0f, 1.0f);
	}
	else
	{
		return (StateIndex == CurrentState) ? 1.0f : 0.0f;
	}
}

bool FAnimNode_StateMachine::IsTransitionActive(int32 TransIndex) const
{
	for (int32 Index = 0; Index < ActiveTransitionArray.Num(); ++Index)
	{
		if (ActiveTransitionArray[Index].SourceTransitionIndices.Contains(TransIndex))
		{
			return true;
		}
	}

	return false;
}

bool FAnimNode_StateMachine::RequestTransitionEvent(const FTransitionEvent& InTransitionEvent)
{
	if (!InTransitionEvent.IsValidRequest() || MaxTransitionsRequests <= 0)
	{
		return false;
	}
	
	const int32 ExistingEventIndex = QueuedTransitionEvents.IndexOfByPredicate([InTransitionEvent](const FTransitionEvent& Transition)
	{
		return Transition.EventName.IsEqual(InTransitionEvent.EventName);
	});

	if (ExistingEventIndex == INDEX_NONE || InTransitionEvent.OverwriteMode == ETransitionRequestOverwriteMode::Append)
	{
		UE_LOGF(LogAnimTransitionRequests, Verbose, "Creating new '%ls' request (Machine: %ls)", *InTransitionEvent.EventName.ToString(), *GetMachineDescription()->MachineName.ToString());

		ensure(QueuedTransitionEvents.Num() <= MaxTransitionsRequests);
		if (QueuedTransitionEvents.Num() == MaxTransitionsRequests)
		{
			UE_LOGF(LogAnimTransitionRequests, Warning, "Transition '%ls' request cap reached, dropping old requests (Machine: %ls)", *InTransitionEvent.EventName.ToString(), *GetMachineDescription()->MachineName.ToString());
			QueuedTransitionEvents.Pop(EAllowShrinking::No);
		}
		QueuedTransitionEvents.Insert(InTransitionEvent, 0);

		return true;
	}
	else if (InTransitionEvent.OverwriteMode == ETransitionRequestOverwriteMode::Ignore)
	{
		UE_LOGF(LogAnimTransitionRequests, Verbose, "Ignoring new '%ls' request (Machine %ls)", *InTransitionEvent.EventName.ToString(), *GetMachineDescription()->MachineName.ToString());
		return false;
	}
	else if (InTransitionEvent.OverwriteMode == ETransitionRequestOverwriteMode::Overwrite)
	{
		UE_LOGF(LogAnimTransitionRequests, Verbose, "Overwriting '%ls' request (Machine %ls)", *InTransitionEvent.EventName.ToString(), *GetMachineDescription()->MachineName.ToString());
		QueuedTransitionEvents[ExistingEventIndex] = InTransitionEvent;
		return true;
	}
	else
	{
		ensure(false);
	}

	return false;
}

void FAnimNode_StateMachine::ClearTransitionEvents(const FName& EventName)
{
	for (int32 RequestIndex = QueuedTransitionEvents.Num() - 1; RequestIndex >= 0; --RequestIndex)
	{
		if (QueuedTransitionEvents[RequestIndex].EventName.IsEqual(EventName))
		{
			UE_LOGF(LogAnimTransitionRequests, Verbose, "Clearing '%ls' request (Machine %ls)", *EventName.ToString(), *GetMachineDescription()->MachineName.ToString());
			QueuedTransitionEvents.RemoveAt(RequestIndex, EAllowShrinking::No);
		}
	}
	QueuedTransitionEvents.Shrink();
}

void FAnimNode_StateMachine::ClearAllTransitionEvents()
{
	UE_LOGF(LogAnimTransitionRequests, Verbose, "Clearing all request (Machine %ls)", *GetMachineDescription()->MachineName.ToString());
	QueuedTransitionEvents.Reset();
}

bool FAnimNode_StateMachine::QueryTransitionEvent(const int32 TransitionIndex, const FName& EventName) const
{
	// Assumes QueuedTransitionEvents is sorted by request creation time, i.e. index 0 is newest request, and that
	// ContainsByPredicate returns the first (newest) request
	return QueuedTransitionEvents.ContainsByPredicate([EventName, TransitionIndex](const FTransitionEvent& Transition)
	{
		return Transition.EventName.IsEqual(EventName) && !Transition.ConsumedTransitions.Contains(TransitionIndex);
	});
}

bool FAnimNode_StateMachine::QueryAndMarkTransitionEvent(const int32 TransitionIndex, const FName& EventName)
{
	// Assumes QueuedTransitionEvents is sorted by request creation time, i.e. index 0 is newest request, and that
	// IndexOfByPredicate returns the first (newest) request
	const int32 EventIndex = QueuedTransitionEvents.IndexOfByPredicate([EventName, TransitionIndex](const FTransitionEvent& Transition)
	{
		return Transition.EventName.IsEqual(EventName) && !Transition.ConsumedTransitions.Contains(TransitionIndex);
	});

	if (EventIndex != INDEX_NONE)
	{
		UE_LOGF(LogAnimTransitionRequests, Verbose, "Marking '%ls' (Machine: %ls, Transition: %d)", *EventName.ToString(), *GetMachineDescription()->MachineName.ToString(), TransitionIndex);
		ensure(!QueuedTransitionEvents[EventIndex].ConsumedTransitions.Contains(TransitionIndex));
		QueuedTransitionEvents[EventIndex].ConsumedTransitions.Add(TransitionIndex);
		return true;
	}
	return false;
}

void FAnimNode_StateMachine::ConsumeMarkedTransitionEvents()
{
	for (int32 RequestIndex = QueuedTransitionEvents.Num() - 1; RequestIndex >= 0; --RequestIndex)
	{
		if (QueuedTransitionEvents[RequestIndex].ToBeConsumed())
		{
			UE_LOGF(LogAnimTransitionRequests, Verbose, "Consuming '%ls' (Machine: %ls)", *QueuedTransitionEvents[RequestIndex].EventName.ToString(), *GetMachineDescription()->MachineName.ToString());
#if WITH_EDITORONLY_DATA
			HandledTransitionEvents.Add(QueuedTransitionEvents[RequestIndex]);
#endif
			QueuedTransitionEvents.RemoveAt(RequestIndex, EAllowShrinking::No);
		}
	}
	QueuedTransitionEvents.Shrink();
}

void FAnimNode_StateMachine::UpdateState(int32 StateIndex, const FAnimationUpdateContext& Context)
{
	if ((StateIndex != INDEX_NONE) && !StatesUpdated.Contains(StateIndex) && !IsAConduitState(StateIndex))
	{
		UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FActiveStateMachineScope> Message(bCreateNotifyMetaData, Context, Context, this, StateIndex);
		StatesUpdated.Add(StateIndex);
		StatePoseLinks[StateIndex].Update(Context);
	}
}


const FPoseContext& FAnimNode_StateMachine::EvaluateState(int32 StateIndex, const FPoseContext& Context)
{
	check(StateCachedPoses.Num() == StatePoseLinks.Num());

	FPoseContext* CachePosePtr = StateCachedPoses[StateIndex];
	if (CachePosePtr == nullptr)
	{
		CachePosePtr = new FPoseContext(Context);
		check(CachePosePtr);
		StateCachedPoses[StateIndex] = CachePosePtr;

		if (!IsAConduitState(StateIndex))
		{
			// Make sure CacheBones has been called before evaluating.
			ConditionallyCacheBonesForState(StateIndex, Context);

			StatePoseLinks[StateIndex].Evaluate(*CachePosePtr);
		}
	}

	return *CachePosePtr;
}

bool FAnimNode_StateMachine::IsAConduitState(int32 StateIndex) const
{
	return ((PRIVATE_MachineDescription != nullptr) && (PRIVATE_MachineDescription->States.IsValidIndex(StateIndex))) ? GetStateInfo(StateIndex).bIsAConduit : false;
}

bool FAnimNode_StateMachine::IsValidTransitionIndex(int32 TransitionIndex) const
{
	return PRIVATE_MachineDescription->Transitions.IsValidIndex(TransitionIndex);
}

FName FAnimNode_StateMachine::GetCurrentStateName() const
{
	if(PRIVATE_MachineDescription->States.IsValidIndex(CurrentState))
	{
		return GetStateInfo().StateName;
	}
	return NAME_None;
}

void FAnimNode_StateMachine::CacheMachineDescription(IAnimClassInterface* AnimBlueprintClass)
{
	PRIVATE_MachineDescription = AnimBlueprintClass->GetBakedStateMachines().IsValidIndex(StateMachineIndexInClass) ? &(AnimBlueprintClass->GetBakedStateMachines()[StateMachineIndexInClass]) : nullptr;
}

#undef LOCTEXT_NAMESPACE
