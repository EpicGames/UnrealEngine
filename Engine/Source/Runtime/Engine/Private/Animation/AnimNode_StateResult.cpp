// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_StateResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_StateResult)

bool FAnimNode_StateResult::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(FAnimNode_Root::StaticStruct()->GetFName()))
	{
		FAnimNode_Root OldValue;
		FAnimNode_Root::StaticStruct()->SerializeItem(Slot, &OldValue, nullptr);
		*static_cast<FAnimNode_Root*>(this) = OldValue;

		return true;
	}

	return false;
}

int32 FAnimNode_StateResult::GetStateIndex() const
{
	return GET_ANIM_NODE_DATA(int32, StateIndex);
}

const FAnimNodeFunctionRef& FAnimNode_StateResult::GetStateEntryFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, StateEntryFunction);
}

const FAnimNodeFunctionRef& FAnimNode_StateResult::GetStateFullyBlendedInFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, StateFullyBlendedInFunction);
}

const FAnimNodeFunctionRef& FAnimNode_StateResult::GetStateExitFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, StateExitFunction);
}

const FAnimNodeFunctionRef& FAnimNode_StateResult::GetStateFullyBlendedOutFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, StateFullyBlendedOutFunction);
}
