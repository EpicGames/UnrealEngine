// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_TransitionPoseEvaluator.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_TransitionPoseEvaluator)

/////////////////////////////////////////////////////
// FAnimNode_TransitionPoseEvaluator

FAnimNode_TransitionPoseEvaluator::FAnimNode_TransitionPoseEvaluator()
	: FramesToCachePose(1)
	, CacheFramesRemaining(1)
	, DataSource(EEvaluatorDataSource::EDS_SourcePose)
	, EvaluatorMode(EEvaluatorMode::EM_Standard)
{
}

void FAnimNode_TransitionPoseEvaluator::SetupCacheFrames()
{
	if (EvaluatorMode == EEvaluatorMode::EM_Freeze)
	{
		// EM_Freeze must evaluate 1 frame to get the initial pose. This cached frame will not call update, only evaluate
		CacheFramesRemaining = 1;
	}
	else if (EvaluatorMode == EEvaluatorMode::EM_DelayedFreeze)
	{
		// EM_DelayedFreeze can evaluate multiple frames, but must evaluate at least one.
		CacheFramesRemaining = FMath::Max(FramesToCachePose, 1);
	}
}

void FAnimNode_TransitionPoseEvaluator::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{	
	FAnimNode_Base::Initialize_AnyThread(Context);
	SetupCacheFrames();
}

void FAnimNode_TransitionPoseEvaluator::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) 
{
	if (!CachedBonesCounter.IsSynchronized_All(Context.AnimInstanceProxy->GetCachedBonesCounter()))
	{
		CachedBonesCounter.SynchronizeWith(Context.AnimInstanceProxy->GetCachedBonesCounter());

		// Pose will be out of date, so reset the evaluation counter
		SetupCacheFrames();
		
		const FBoneContainer& RequiredBone = Context.AnimInstanceProxy->GetRequiredBones();
		CachedPose.SetBoneContainer(&RequiredBone);
		CachedCurve.InitFrom(RequiredBone);
	}
}

void FAnimNode_TransitionPoseEvaluator::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	if (!CachedBonesCounter.IsSynchronized_All(Context.AnimInstanceProxy->GetCachedBonesCounter()))
	{
		CachedBonesCounter.SynchronizeWith(Context.AnimInstanceProxy->GetCachedBonesCounter());

		// Pose will be out of date, so reset the # of cached frames
		SetupCacheFrames();
	}

	// updating is all handled in state machine
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Cached Frames Remaining"), CacheFramesRemaining);
}

void FAnimNode_TransitionPoseEvaluator::Evaluate_AnyThread(FPoseContext& Output)
{	
	// the cached pose is evaluated in the state machine and set via CachePose(). 
	// This is because we need information about the transition that is not available at this level
	Output.Pose.CopyBonesFrom(CachedPose);
	Output.Curve.CopyFrom(CachedCurve);
	Output.CustomAttributes.CopyFrom(CachedAttributes);

	if ((EvaluatorMode != EEvaluatorMode::EM_Standard) && (CacheFramesRemaining > 0))
	{
		CacheFramesRemaining = FMath::Max(CacheFramesRemaining - 1, 0);
	}
}

void FAnimNode_TransitionPoseEvaluator::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += FString::Printf(TEXT("(Cached Frames Remaining: %i)"), CacheFramesRemaining);
	DebugData.AddDebugItem(DebugLine);
}

bool FAnimNode_TransitionPoseEvaluator::InputNodeNeedsUpdate(const FAnimationUpdateContext& Context) const
{
	// EM_Standard mode always updates and EM_DelayedFreeze mode only updates if there are cache frames remaining
	return (EvaluatorMode == EEvaluatorMode::EM_Standard) || ((EvaluatorMode == EEvaluatorMode::EM_DelayedFreeze) && (CacheFramesRemaining > 0)) || !CachedBonesCounter.IsSynchronized_All(Context.AnimInstanceProxy->GetCachedBonesCounter());
}

bool FAnimNode_TransitionPoseEvaluator::InputNodeNeedsEvaluate() const
{
	return (EvaluatorMode == EEvaluatorMode::EM_Standard) || (CacheFramesRemaining > 0);
}

void FAnimNode_TransitionPoseEvaluator::CachePose(const FPoseContext& PoseToCache)
{
	CachedPose.CopyBonesFrom(PoseToCache.Pose);
	CachedPose.NormalizeRotations();
	CachedCurve.CopyFrom(PoseToCache.Curve);
	CachedAttributes.CopyFrom(PoseToCache.CustomAttributes);
}

