// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_TransitionResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_TransitionResult)

/////////////////////////////////////////////////////
// FAnimNode_TransitionResult

FAnimNode_TransitionResult::FAnimNode_TransitionResult()
	: bCanEnterTransition(false)
{
}

void FAnimNode_TransitionResult::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_Base::Initialize_AnyThread(Context);
}

void FAnimNode_TransitionResult::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) 
{
}

void FAnimNode_TransitionResult::Update_AnyThread(const FAnimationUpdateContext& Context)
{
}

void FAnimNode_TransitionResult::Evaluate_AnyThread(FPoseContext& Output)
{
}


void FAnimNode_TransitionResult::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugData.AddDebugItem(DebugLine);
}

