// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNode_UseCachedPose.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_UseCachedPose)

/////////////////////////////////////////////////////
// FAnimNode_UseCachedPose

FAnimNode_UseCachedPose::FAnimNode_UseCachedPose()
{
}

void FAnimNode_UseCachedPose::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_Base::Initialize_AnyThread(Context);

	LinkToCachingNode.Initialize(Context);
}

void FAnimNode_UseCachedPose::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) 
{
	LinkToCachingNode.CacheBones(Context);
}

void FAnimNode_UseCachedPose::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	{
		// This makes sure we dont see a 'save cached pose' entry in the debug data.
		// This will be handled later on when the cached pose branch gets taken.
		TRACE_SCOPED_ANIM_NODE_SUSPEND;	
		LinkToCachingNode.Update(Context);
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), CachePoseName);
}

void FAnimNode_UseCachedPose::Evaluate_AnyThread(FPoseContext& Output)
{
	LinkToCachingNode.Evaluate(Output);
}

void FAnimNode_UseCachedPose::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("%s:"), *CachePoseName.ToString());

	DebugData.AddDebugItem(DebugLine, true);

	// we explicitly do not forward this call to the SaveCachePose node here.
	// It is handled in FAnimInstanceProxy::GatherDebugData
}

