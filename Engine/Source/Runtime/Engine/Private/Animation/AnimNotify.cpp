// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotify.h"

#include "Animation/AnimNotifyQueue.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimSequenceBase.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify)

/////////////////////////////////////////////////////
// UAnimNotify

UAnimNotify::UAnimNotify(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MeshContext(NULL)
{

#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(255, 200, 200, 255);
	bShouldFireInEditor = true;
#endif // WITH_EDITORONLY_DATA

	bIsNativeBranchingPoint = false;
}

void UAnimNotify::Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	USkeletalMeshComponent* PrevContext = MeshContext;
	MeshContext = MeshComp;
	Received_Notify(MeshComp, Animation, EventReference);
	MeshContext = PrevContext;
}

void UAnimNotify::BranchingPointNotify(FBranchingPointNotifyPayload& BranchingPointPayload)
{
	const FAnimNotifyEventReference EventReference;
	Notify(BranchingPointPayload.SkelMeshComponent, BranchingPointPayload.SequenceAsset, EventReference);
}

class UWorld* UAnimNotify::GetWorld() const
{
	return (MeshContext ? MeshContext->GetWorld() : NULL);
}

/// @cond DOXYGEN_WARNINGS

FString UAnimNotify::GetNotifyName_Implementation() const
{
	FString NotifyName;

#if WITH_EDITORONLY_DATA
	if (UObject* ClassGeneratedBy = GetClass()->ClassGeneratedBy)
	{
		// GeneratedBy will be valid for blueprint types and gives a clean name without a suffix
		NotifyName = ClassGeneratedBy->GetName();
	}
	else
#endif
	{
		// Native notify classes are clean without a suffix otherwise
		NotifyName = GetClass()->GetName();
	}

	NotifyName.ReplaceInline(TEXT("AnimNotify_"), TEXT(""), ESearchCase::CaseSensitive);
	
	return NotifyName;
}

float UAnimNotify::GetDefaultTriggerWeightThreshold_Implementation() const
{
	return ZERO_ANIMWEIGHT_THRESH;
}

/// @endcond

void UAnimNotify::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	// Ensure that all loaded notifies are transactional
	SetFlags(GetFlags() | RF_Transactional);

	// Make sure the asset isn't bogus (e.g., a looping particle system in a one-shot notify)
	ValidateAssociatedAssets();
#endif
}

void UAnimNotify::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	ValidateAssociatedAssets();
#endif
	Super::PreSave(ObjectSaveContext);
}

UObject* UAnimNotify::GetContainingAsset() const
{
	UObject* ContainingAsset = GetTypedOuter<UAnimSequenceBase>();
	if (ContainingAsset == nullptr)
	{
		ContainingAsset = GetOutermost();
	}
	return ContainingAsset;
}
