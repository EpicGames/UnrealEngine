// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifyEndDataContext.h"
#include "Animation/AnimNodeMessages.h"

namespace UE::Anim {

IMPLEMENT_NOTIFY_CONTEXT_INTERFACE(FAnimNotifyEndDataContext)

FAnimNotifyEndDataContext::FAnimNotifyEndDataContext(bool bInReachedEnd)
	: bReachedEnd(bInReachedEnd)
{
}
	
}	// namespace UE::Anim
