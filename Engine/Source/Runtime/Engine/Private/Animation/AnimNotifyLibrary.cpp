// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifyLibrary.h"
#include "Animation/AnimNodeMessages.h"
#include "Animation/AnimNotifyQueue.h"
#include "Animation/AnimNotifyEndDataContext.h"
#include "Animation/AnimSequenceBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyLibrary)

bool UAnimNotifyLibrary::NotifyStateReachedEnd(const FAnimNotifyEventReference& EventReference)
{
	const UE::Anim::FAnimNotifyEndDataContext* EndData = EventReference.GetContextData<UE::Anim::FAnimNotifyEndDataContext>();

	if (EndData != nullptr)
	{
		return EndData->bReachedEnd;
	}

	return false;
}

float UAnimNotifyLibrary::GetNotifyStartTime(const FAnimNotifyEventReference& EventReference)
{
	if (const FAnimNotifyEvent* NotifyEvent = EventReference.GetNotify())
	{
		return NotifyEvent->GetTriggerTime();
	}

	return 0.0f;
}

float UAnimNotifyLibrary::GetCurrentAnimationTime(const FAnimNotifyEventReference& EventReference)
{
	return EventReference.GetCurrentAnimationTime();
}

float UAnimNotifyLibrary::GetCurrentAnimationTimeRatio(const FAnimNotifyEventReference& EventReference)
{
	if(const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(EventReference.GetSourceObject()))
	{
		if(AnimSequenceBase->GetPlayLength() > 0.0f)
		{
			return FMath::Clamp(EventReference.GetCurrentAnimationTime() / AnimSequenceBase->GetPlayLength(), 0.0f, 1.0f);
		}
	}

	return 0.0f;
}

float UAnimNotifyLibrary::GetCurrentAnimationNotifyStateTime(const FAnimNotifyEventReference& EventReference)
{
	if(const FAnimNotifyEvent* NotifyEvent = EventReference.GetNotify())
	{
		return FMath::Clamp(EventReference.GetCurrentAnimationTime() - NotifyEvent->GetTriggerTime(), 0.0f, NotifyEvent->GetDuration());
	}

	return 0.0f;
}

float UAnimNotifyLibrary::GetCurrentAnimationNotifyStateTimeRatio(const FAnimNotifyEventReference& EventReference)
{
	if(const FAnimNotifyEvent* NotifyEvent = EventReference.GetNotify())
	{
		if(NotifyEvent->GetDuration() > 0.0f)
		{
			return FMath::Clamp((EventReference.GetCurrentAnimationTime() - NotifyEvent->GetTriggerTime()) / NotifyEvent->GetDuration(), 0.0f, 1.0f);
		}
	}

	return 0.0f;
}

bool UAnimNotifyLibrary::IsBlendingOut(const FAnimNotifyEventReference& EventReference)
{
	return !EventReference.IsActiveContext();
}