// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifyMirrorInspectionLibrary.h"
#include "Animation/AnimNodeMessages.h"
#include "Animation/AnimNotifyQueue.h"
#include "Animation/MirrorSyncScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyMirrorInspectionLibrary)

bool UAnimNotifyMirrorInspectionLibrary::IsTriggeredByMirroredAnimation(const FAnimNotifyEventReference& EventReference)
{
	const UE::Anim::FAnimNotifyMirrorContext * MirrorContext = EventReference.GetContextData<UE::Anim::FAnimNotifyMirrorContext>();
	if (MirrorContext)
	{
		return MirrorContext->bAnimationMirrored; 
	}
	return false; 
}

const UMirrorDataTable* UAnimNotifyMirrorInspectionLibrary::GetMirrorDataTable(const FAnimNotifyEventReference& EventReference)
{
	const UE::Anim::FAnimNotifyMirrorContext * MirrorContext =  EventReference.GetContextData<UE::Anim::FAnimNotifyMirrorContext>();
	if (MirrorContext)
	{
		return MirrorContext->MirrorTable.Get(); 
	}
	return nullptr;
}
