// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifyQueue.h"

#include "Animation/AnimEventsFilterScope.h"
#include "Animation/AnimInstanceProxy.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyQueue)

bool operator==(const FAnimNotifyEventReference& Lhs, const FAnimNotifyEvent& Rhs)
{
	if (Lhs.NotifySource && Lhs.Notify)
	{
		return (*(Lhs.Notify)) == Rhs;
	}
	return false;
}

void FAnimNotifyEventReference::GatherTickRecordData(const FAnimTickRecord& InTickRecord)
{
	ContextData = InTickRecord.ContextData;
	CurrentAnimTime = InTickRecord.TimeAccumulator != nullptr ? *InTickRecord.TimeAccumulator : 0.0f;
	bActiveContext = InTickRecord.bActiveContext;
}

bool FAnimNotifyQueue::PassesFiltering(const FAnimNotifyEvent* Notify) const
{
	switch (Notify->NotifyFilterType)
	{
		case ENotifyFilterType::NoFiltering:
		{
			return true;
		}
		case ENotifyFilterType::LOD:
		{
			return Notify->NotifyFilterLOD > PredictedLODLevel;
		}
		default:
		{
			ensure(false); // Unknown Filter Type
		}
	}
	return true;
}

bool FAnimNotifyQueue::PassesChanceOfTriggering(const FAnimNotifyEvent* Event) const
{
	return Event->NotifyStateClass ? true : RandomStream.FRandRange(0.f, 1.f) < Event->NotifyTriggerChance;
}

void FAnimNotifyQueue::AddAnimNotifiesToDest(bool bSrcIsLeader, const TArray<FAnimNotifyEventReference>& NewNotifies, TArray<FAnimNotifyEventReference>& DestArray, const float InstanceWeight)
{
	bool bIsDedicatedServer = World.IsValid() ? (World->GetNetMode() == NM_DedicatedServer) : false;
	// for now there is no filter whatsoever, it just adds everything requested
	for (const FAnimNotifyEventReference& NotifyRef : NewNotifies)
	{
		const FAnimNotifyEvent* Notify = NotifyRef.GetNotify();
		if( Notify && (bSrcIsLeader || Notify->bTriggerOnFollower))
		{
			// only add if it is over TriggerWeightThreshold
			const bool bPassesDedicatedServerCheck = Notify->bTriggerOnDedicatedServer || !bIsDedicatedServer;
			if (bPassesDedicatedServerCheck && Notify->TriggerWeightThreshold <= InstanceWeight && PassesFiltering(Notify) && PassesChanceOfTriggering(Notify))
			{
				const UE::Anim::IAnimEventsFilterContext* ContextData = Notify->bCanBeFilteredViaRequest ? NotifyRef.GetContextData<UE::Anim::IAnimEventsFilterContext>() : nullptr;
				const bool bPassedScopeFilter = ContextData ? !ContextData->ShouldFilterNotify(NotifyRef) : true;

				if (bPassedScopeFilter)
				{
					// Only add unique AnimNotifyState instances just once. We can get multiple triggers if looping over an animation.
					// It is the same state, so just report it once.
					Notify->NotifyStateClass ? DestArray.AddUnique(NotifyRef) : DestArray.Add(NotifyRef);	
				}
			}
		}
	}
}

void FAnimNotifyQueue::AddAnimNotifiesToDestNoFiltering(const TArray<FAnimNotifyEventReference>& NewNotifies, TArray<FAnimNotifyEventReference>& DestArray) const
{
	for (const FAnimNotifyEventReference& NotifyRef : NewNotifies)
	{
		if (const FAnimNotifyEvent* Notify = NotifyRef.GetNotify())
		{
			Notify->NotifyStateClass ? DestArray.AddUnique(NotifyRef) : DestArray.Add(NotifyRef);
		}
	}
}

void FAnimNotifyQueue::AddAnimNotify(const FAnimNotifyEvent* Notify, const UObject* NotifySource)
{
	AnimNotifies.Add(FAnimNotifyEventReference(Notify, NotifySource));
}

void FAnimNotifyQueue::AddAnimNotifies(bool bSrcIsLeader, const TArray<FAnimNotifyEventReference>& NewNotifies, const float InstanceWeight)
{
	AddAnimNotifiesToDest(bSrcIsLeader, NewNotifies, AnimNotifies, InstanceWeight);
}

void FAnimNotifyQueue::AddAnimNotifies(bool bSrcIsLeader, const TMap<FName, TArray<FAnimNotifyEventReference>>& NewNotifies, const float InstanceWeight)
{
	for (const TPair<FName, TArray<FAnimNotifyEventReference>>& Pair : NewNotifies)
	{
		TArray<FAnimNotifyEventReference>& Notifies = UnfilteredMontageAnimNotifies.FindOrAdd(Pair.Key).Notifies;
		AddAnimNotifiesToDest(bSrcIsLeader, Pair.Value, Notifies, InstanceWeight);
	}
}

void FAnimNotifyQueue::Reset(USkeletalMeshComponent* Component)
{
	AnimNotifies.Reset();
	UnfilteredMontageAnimNotifies.Reset();
	if (Component)
	{
		PredictedLODLevel = Component->GetPredictedLODLevel();
		World = Component->GetWorld();
	}
	else
	{
		PredictedLODLevel =  -1;
		World = nullptr;
	}
}

void FAnimNotifyQueue::Append(const FAnimNotifyQueue& Queue)
{
	// we dont just append here - we need to preserve uniqueness for AnimNotifyState instances
	AddAnimNotifiesToDestNoFiltering(Queue.AnimNotifies, AnimNotifies);

	for (const TPair<FName, FAnimNotifyArray>& Pair : Queue.UnfilteredMontageAnimNotifies)
	{
		TArray<FAnimNotifyEventReference>& Notifies = UnfilteredMontageAnimNotifies.FindOrAdd(Pair.Key).Notifies;
		AddAnimNotifiesToDestNoFiltering(Pair.Value.Notifies, Notifies);
	}
}

void FAnimNotifyQueue::ApplyMontageNotifies(const FAnimInstanceProxy& Proxy)
{
	for (const TPair<FName, FAnimNotifyArray>& Pair : UnfilteredMontageAnimNotifies)
	{
		if (Proxy.IsSlotNodeRelevantForNotifies(Pair.Key))
		{
			AddAnimNotifiesToDestNoFiltering(Pair.Value.Notifies, AnimNotifies);
		}
	}
	UnfilteredMontageAnimNotifies.Reset();
}

