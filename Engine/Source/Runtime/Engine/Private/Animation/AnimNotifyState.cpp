// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifyEndDataContext.h"
#include "Animation/AnimNotifyQueue.h"
#include "UObject/ObjectSaveContext.h"
#include "Animation/AnimSequenceBase.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState)

/////////////////////////////////////////////////////
// UAnimNotifyState

UAnimNotifyState::UAnimNotifyState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(200, 200, 255, 255);
	bShouldFireInEditor = true;
#endif // WITH_EDITORONLY_DATA

	bIsNativeBranchingPoint = false;
	NotifyStateBehaviorFlags = 0;
}

void UAnimNotifyState::NotifyBegin(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference)
{
	Received_NotifyBegin(MeshComp, Animation, TotalDuration, EventReference);
}

void UAnimNotifyState::NotifyTick(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference)
{
	Received_NotifyTick(MeshComp, Animation, FrameDeltaTime, EventReference);
}

void UAnimNotifyState::NotifyEnd(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	Received_NotifyEnd(MeshComp, Animation, EventReference);
}

void UAnimNotifyState::BranchingPointNotifyBegin(FBranchingPointNotifyPayload& BranchingPointPayload)
{
	const FAnimNotifyEventReference EventReference;
	NotifyBegin(BranchingPointPayload.SkelMeshComponent, BranchingPointPayload.SequenceAsset, BranchingPointPayload.NotifyEvent ? BranchingPointPayload.NotifyEvent->GetDuration() : 0.f, EventReference);
}

void UAnimNotifyState::BranchingPointNotifyTick(FBranchingPointNotifyPayload& BranchingPointPayload, float FrameDeltaTime)
{
	const FAnimNotifyEventReference EventReference;
	NotifyTick(BranchingPointPayload.SkelMeshComponent, BranchingPointPayload.SequenceAsset, FrameDeltaTime, EventReference);
}

void UAnimNotifyState::BranchingPointNotifyEnd(FBranchingPointNotifyPayload& BranchingPointPayload)
{
	FAnimNotifyEventReference EventReference;
	if (BranchingPointPayload.bReachedEnd)
	{
		EventReference.AddContextData<UE::Anim::FAnimNotifyEndDataContext>(true);
	}

	NotifyEnd(BranchingPointPayload.SkelMeshComponent, BranchingPointPayload.SequenceAsset, EventReference);
}

/// @cond DOXYGEN_WARNINGS

FString UAnimNotifyState::GetNotifyName_Implementation() const
{
	FString NotifyName;
	
#if WITH_EDITORONLY_DATA
	if (UObject* ClassGeneratedBy = GetClass()->ClassGeneratedBy)
	{
		// GeneratedBy will be valid for blueprint types and gives a clean name without a suffix
		NotifyName = ClassGeneratedBy->GetName();
	}
	else
#endif
	{
		// Native notify classes are clean without a suffix otherwise
		NotifyName = GetClass()->GetName();
	}

	NotifyName.ReplaceInline(TEXT("AnimNotifyState_"), TEXT(""), ESearchCase::CaseSensitive);
	
	return NotifyName;
}

/// @endcond

float UAnimNotifyState::GetDefaultTriggerWeightThreshold_Implementation() const
{
	return ZERO_ANIMWEIGHT_THRESH;
}

void UAnimNotifyState::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	// Ensure that all loaded notifies are transactional
	SetFlags(GetFlags() | RF_Transactional);

	// Make sure the asset isn't bogus (e.g., a looping particle system in a one-shot notify)
	ValidateAssociatedAssets();
#endif
}

void UAnimNotifyState::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	ValidateAssociatedAssets();
#endif
	Super::PreSave(ObjectSaveContext);
}

UObject* UAnimNotifyState::GetContainingAsset() const
{
	UObject* ContainingAsset = GetTypedOuter<UAnimSequenceBase>();
	if (ContainingAsset == nullptr)
	{
		ContainingAsset = GetOutermost();
	}
	return ContainingAsset;
}
