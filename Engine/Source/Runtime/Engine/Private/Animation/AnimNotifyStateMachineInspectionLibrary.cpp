// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifyStateMachineInspectionLibrary.h"
#include "Animation/AnimNode_StateMachine.h"
#include "Animation/ActiveStateMachineScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyStateMachineInspectionLibrary)

bool UAnimNotifyStateMachineInspectionLibrary::IsStateMachineInEventContext(const FAnimNotifyEventReference& Reference, int32 StateMachineIndex)
{
	const UE::Anim::FAnimNotifyStateMachineContext * StateMachineContext = Reference.GetContextData<UE::Anim::FAnimNotifyStateMachineContext>();
	if (StateMachineContext)
	{
		if (StateMachineContext->IsStateMachineInContext(StateMachineIndex))
		{
			return true; 
		}
	}
	return false; 
}

bool UAnimNotifyStateMachineInspectionLibrary::IsStateInStateMachineInEventContext(const FAnimNotifyEventReference& Reference, int32 StateMachineIndex, int32 StateIndex) 
{
	const UE::Anim::FAnimNotifyStateMachineContext * StateMachineContext = Reference.GetContextData<UE::Anim::FAnimNotifyStateMachineContext>();
	if (StateMachineContext)
	{
		if (StateMachineContext->IsStateInStateMachineInContext(StateMachineIndex, StateIndex))
		{
			return true; 
		}
	}
	return false;
}

bool UAnimNotifyStateMachineInspectionLibrary::IsTriggeredByStateMachine(const FAnimNotifyEventReference& EventReference, UAnimInstance* AnimInstance, FName StateMachineName)
{
	if (AnimInstance)
	{
		int32 MachineIndex;
		const FBakedAnimationStateMachine* BakedAnimationStateMachine = nullptr;
		AnimInstance->GetStateMachineIndexAndDescription(StateMachineName,MachineIndex, &BakedAnimationStateMachine);
		if(MachineIndex != INDEX_NONE)
		{
			return IsStateMachineInEventContext(EventReference, MachineIndex);
		}
	}
	return false; 
}

bool UAnimNotifyStateMachineInspectionLibrary::IsTriggeredByStateInStateMachine(const FAnimNotifyEventReference& EventReference, UAnimInstance* AnimInstance, FName StateMachineName, FName StateName)
{
	if (AnimInstance)
	{
		int32 MachineIndex;
		const FBakedAnimationStateMachine* BakedAnimationStateMachine = nullptr;
		AnimInstance->GetStateMachineIndexAndDescription(StateMachineName, MachineIndex, &BakedAnimationStateMachine);
		if(MachineIndex != INDEX_NONE && BakedAnimationStateMachine)
		{
			int32 StateIndex = BakedAnimationStateMachine->FindStateIndex(StateName);
			return IsStateInStateMachineInEventContext(EventReference, MachineIndex, StateIndex);
		}
	}
	return false; 
}

bool  UAnimNotifyStateMachineInspectionLibrary::IsTriggeredByState(const FAnimNotifyEventReference& EventReference, UAnimInstance* AnimInstance, FName StateName)
{
	if (AnimInstance)
	{
		const UE::Anim::FAnimNotifyStateMachineContext * StateMachineContext = EventReference.GetContextData<UE::Anim::FAnimNotifyStateMachineContext>();
		if (StateMachineContext)
		{
			for (const FEncounteredStateMachineStack::FStateMachineEntry& Entry: StateMachineContext->EncounteredStateMachines.StateStack)
			{
				const FAnimNode_StateMachine* StateMachineInstance = AnimInstance->GetStateMachineInstance(Entry.StateMachineIndex);
				if(StateMachineInstance->GetStateInfo(Entry.StateIndex).StateName == StateName)
				{
					return true; 
				}
			}
		}
	}
	return false;
}

