// Copyright Epic Games, Inc. All Rights Reserved.
#include "Animation/AnimNotifies/AnimNotifyState_DisableRootMotion.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_DisableRootMotion)

UAnimNotifyState_DisableRootMotion::UAnimNotifyState_DisableRootMotion(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsNativeBranchingPoint = true;
}

void UAnimNotifyState_DisableRootMotion::BranchingPointNotifyBegin(FBranchingPointNotifyPayload& BranchingPointPayload)
{
	Super::BranchingPointNotifyBegin(BranchingPointPayload);

	if (USkeletalMeshComponent* MeshComp = BranchingPointPayload.SkelMeshComponent)
	{
		if (UAnimInstance* AnimInstance = MeshComp->GetAnimInstance())
		{
			if (FAnimMontageInstance* MontageInstance = AnimInstance->GetMontageInstanceForID(BranchingPointPayload.MontageInstanceID))
			{
				MontageInstance->PushDisableRootMotion();
			}
		}
	}
}

void UAnimNotifyState_DisableRootMotion::BranchingPointNotifyEnd(FBranchingPointNotifyPayload& BranchingPointPayload)
{
	Super::BranchingPointNotifyEnd(BranchingPointPayload);

	if (USkeletalMeshComponent* MeshComp = BranchingPointPayload.SkelMeshComponent)
	{
		if (UAnimInstance* AnimInstance = MeshComp->GetAnimInstance())
		{
			if (FAnimMontageInstance* MontageInstance = AnimInstance->GetMontageInstanceForID(BranchingPointPayload.MontageInstanceID))
			{
				MontageInstance->PopDisableRootMotion();
			}
		}
	}
}

#if WITH_EDITOR
bool UAnimNotifyState_DisableRootMotion::CanBePlaced(UAnimSequenceBase* Animation) const
{
	return (Animation && Animation->IsA(UAnimMontage::StaticClass()));
}
#endif // WITH_EDITOR
