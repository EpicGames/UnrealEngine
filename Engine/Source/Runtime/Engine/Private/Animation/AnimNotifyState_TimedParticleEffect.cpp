// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotifyState_TimedParticleEffect.h"
#include "Components/SkeletalMeshComponent.h"
#include "Particles/ParticleSystem.h"
#include "Particles/ParticleSystemComponent.h"
#include "Kismet/GameplayStatics.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_TimedParticleEffect)

UAnimNotifyState_TimedParticleEffect::UAnimNotifyState_TimedParticleEffect(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	PSTemplate = nullptr;
	LocationOffset.Set(0.0f, 0.0f, 0.0f);
	RotationOffset = FRotator(0.0f, 0.0f, 0.0f);
}

void UAnimNotifyState_TimedParticleEffect::NotifyBegin(USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference)
{
	// Only spawn if we've got valid params
	if(ValidateParameters(MeshComp))
	{
		UParticleSystemComponent* NewComponent = UGameplayStatics::SpawnEmitterAttached(PSTemplate, MeshComp, SocketName, LocationOffset, RotationOffset, EAttachLocation::KeepRelativeOffset, !bDestroyAtEnd);
	}
	Received_NotifyBegin(MeshComp, Animation, TotalDuration, EventReference);
}

void UAnimNotifyState_TimedParticleEffect::NotifyTick(USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference)
{
	Received_NotifyTick(MeshComp, Animation, FrameDeltaTime, EventReference);
}

void UAnimNotifyState_TimedParticleEffect::NotifyEnd(USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, const FAnimNotifyEventReference& EventReference)
{
	TArray<USceneComponent*> Children;
	MeshComp->GetChildrenComponents(false, Children);

	for(USceneComponent* Component : Children)
	{
		if(UParticleSystemComponent* ParticleComponent = Cast<UParticleSystemComponent>(Component))
		{
			bool bSocketMatch = ParticleComponent->GetAttachSocketName() == SocketName;
			bool bTemplateMatch = ParticleComponent->Template == PSTemplate;

#if WITH_EDITORONLY_DATA
			// In editor someone might have changed our parameters while we're ticking; so check 
			// previous known parameters too.
			bSocketMatch |= PreviousSocketNames.Contains(ParticleComponent->GetAttachSocketName());
			bTemplateMatch |= PreviousPSTemplates.Contains(ParticleComponent->Template);
#endif

			if(bSocketMatch && bTemplateMatch && !ParticleComponent->bWasDeactivated)
			{
				// Either destroy the component or deactivate it to have it's active particles finish.
				// The component will auto destroy once all particle are gone.
				if(bDestroyAtEnd)
				{
					ParticleComponent->DestroyComponent();
				}
				else
				{
					ParticleComponent->DeactivateSystem();
				}

#if WITH_EDITORONLY_DATA
				// No longer need to track previous values as we've found our component
				// and removed it.
				PreviousPSTemplates.Empty();
				PreviousSocketNames.Empty();
#endif
				// Removed a component, no need to continue
				break;
			}
		}
	}

	Received_NotifyEnd(MeshComp, Animation, EventReference);
}

bool UAnimNotifyState_TimedParticleEffect::ValidateParameters(USkeletalMeshComponent* MeshComp)
{
	bool bValid = true;

	if(!PSTemplate)
	{
		bValid = false;
	}
	else if(!MeshComp->DoesSocketExist(SocketName) && MeshComp->GetBoneIndex(SocketName) == INDEX_NONE)
	{
		bValid = false;
	}

	return bValid;
}

FString UAnimNotifyState_TimedParticleEffect::GetNotifyName_Implementation() const
{
	if(PSTemplate)
	{
		return PSTemplate->GetName();
	}

	return UAnimNotifyState::GetNotifyName_Implementation();
}

#if WITH_EDITOR
void UAnimNotifyState_TimedParticleEffect::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if(PropertyAboutToChange)
	{
		if(PropertyAboutToChange->GetName() == GET_MEMBER_NAME_STRING_CHECKED(UAnimNotifyState_TimedParticleEffect, PSTemplate) && PSTemplate != NULL)
		{
			PreviousPSTemplates.Add(PSTemplate);
		}

		if(PropertyAboutToChange->GetName() == GET_MEMBER_NAME_STRING_CHECKED(UAnimNotifyState_TimedParticleEffect, SocketName) && SocketName != FName("None"))
		{
			PreviousSocketNames.Add(SocketName);
		}
	}
}
#endif
