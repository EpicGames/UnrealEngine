// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotifyState_Trail.h"
#include "Animation/AnimSequenceBase.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"
#include "ParticleEmitterInstances.h"
#include "Particles/ParticleSystem.h"
#include "Particles/ParticleSystemComponent.h"
#include "Particles/TypeData/ParticleModuleTypeDataAnimTrail.h"
#include "Animation/AnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_Trail)

#define LOCTEXT_NAMESPACE "AnimNotifyState_Trail"

DEFINE_LOG_CATEGORY(LogAnimTrails);

typedef TInlineComponentArray<UParticleSystemComponent*, 8> ParticleSystemComponentArray;

static void GetCandidateSystems(USkeletalMeshComponent& MeshComp, ParticleSystemComponentArray& Components)
{
	if (AActor* Owner = MeshComp.GetOwner())
	{
		Owner->GetComponents(Components);
	}
	else
	{
		// No actor owner in some editor windows. Get PSCs spawned by the MeshComp.
		ForEachObjectWithOuter(&MeshComp, [&Components](UObject* Child)
		{
			if (UParticleSystemComponent* ChildPSC = Cast<UParticleSystemComponent>(Child))
			{
				Components.Add(ChildPSC);
			}
		}, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);
	}
}

/////////////////////////////////////////////////////
// UAnimNotifyState_Trail

UAnimNotifyState_Trail::UAnimNotifyState_Trail(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PSTemplate = NULL;
	FirstSocketName = NAME_None;
	SecondSocketName = NAME_None;
	WidthScaleMode = ETrailWidthMode_FromCentre;
	WidthScaleCurve = NAME_None;

	bRecycleSpawnedSystems = true;

#if WITH_EDITORONLY_DATA
	bRenderGeometry = true;
	bRenderSpawnPoints = false;
	bRenderTangents = false;
	bRenderTessellation = false;
#endif // WITH_EDITORONLY_DATA
}

UParticleSystem* UAnimNotifyState_Trail::GetOverridenPSTemplate(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation) const
{
	return OverridePSTemplate(MeshComp, Animation);
}

float UAnimNotifyState_Trail::GetCurveWidth(USkeletalMeshComponent* MeshComp) const
{
	UAnimInstance* AnimInst = MeshComp->GetAnimInstance();
	float Width = 1.0f;
	if (WidthScaleCurve != NAME_None && AnimInst)
	{
		if (!AnimInst->GetCurveValue(WidthScaleCurve, Width))
		{
			// Fallback to 1.f if curve was not found
			Width = 1.f;
		}
	}
	return Width;
}

void UAnimNotifyState_Trail::NotifyBegin(class USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference)
{
	bool bError = ValidateInput(MeshComp);

	if (MeshComp->GetWorld()->GetNetMode() == NM_DedicatedServer)
	{
		return;
	}

	UParticleSystem* ParticleSystemTemplate = GetOverridenPSTemplate(MeshComp, Animation);
	if (ParticleSystemTemplate != nullptr)
	{
		PSTemplate = ParticleSystemTemplate;
	}

	if(PSTemplate == nullptr)
	{
		UE_LOGF(LogParticles, Warning, "Trail Notify: Null PSTemplate for trail notify in anim: %ls", *GetPathNameSafe(Animation));
		return;
	}

	ParticleSystemComponentArray Children;
	GetCandidateSystems(*MeshComp, Children);

	float Width = GetCurveWidth(MeshComp);

	UParticleSystemComponent* RecycleCandidates[3] = {nullptr, nullptr, nullptr}; // in order of priority
	bool bFoundExistingTrail = false;
	for (UParticleSystemComponent* ParticleComp : Children)
	{
		if (ParticleComp->IsActive())
		{
			UParticleSystemComponent::TrailEmitterArray TrailEmitters;
			ParticleComp->GetOwnedTrailEmitters(TrailEmitters, this, false);

			if (TrailEmitters.Num() > 0)
			{
				// This has active emitters, we'll just restart this one.
				bFoundExistingTrail = true;

				//If there are any trails, ensure the template hasn't been changed. Also destroy the component if there are errors.
				if (bError || (PSTemplate != ParticleComp->Template && ParticleComp->GetOuter() == MeshComp))
				{
					//The PSTemplate was changed so we need to destroy this system and create it again with the new template. May be able to just change the template?
					ParticleComp->DestroyComponent();
				}
				else
				{
					for (FParticleAnimTrailEmitterInstance* Trail : TrailEmitters)
					{
						Trail->BeginTrail();
						Trail->SetTrailSourceData(FirstSocketName, SecondSocketName, WidthScaleMode, Width);

	#if WITH_EDITORONLY_DATA
						Trail->SetTrailDebugData(bRenderGeometry, bRenderSpawnPoints, bRenderTessellation, bRenderTangents);
	#endif
					}
				}

				break;
			}
		}
		else if (ParticleComp->bAllowRecycling && !ParticleComp->IsActive())
		{
			// We prefer to recycle one with a matching template, and prefer one created by us.
			// 0: matching template, owned by mesh
			// 1: matching template, owned by actor
			// 2: non-matching template, owned by actor or mesh
			int32 RecycleIndex = 2;
			if (ParticleComp->Template == PSTemplate)
			{
				RecycleIndex = (ParticleComp->GetOuter() == MeshComp ? 0 : 1);
			}
			RecycleCandidates[RecycleIndex] = ParticleComp;
		}
	}

	if (!bFoundExistingTrail && !bError)
	{
		// Spawn a new component from PSTemplate, or recycle an old one.
		UParticleSystemComponent* RecycleComponent = (RecycleCandidates[0] ? RecycleCandidates[0] : (RecycleCandidates[1] ? RecycleCandidates[1] : RecycleCandidates[2]));
		UParticleSystemComponent* NewParticleComp = (RecycleComponent ? RecycleComponent : NewObject<UParticleSystemComponent>(MeshComp));
		NewParticleComp->bAutoDestroy = (RecycleComponent ? false : !bRecycleSpawnedSystems);
		NewParticleComp->bAllowRecycling = true;
		NewParticleComp->SecondsBeforeInactive = 0.0f;
		NewParticleComp->bAutoActivate = false;
		NewParticleComp->bOverrideLODMethod = false;
		NewParticleComp->SetRelativeScale3D_Direct(FVector(1.f));
		NewParticleComp->bAutoManageAttachment = true; // Let it detach when finished (only happens if not auto-destroying)
		NewParticleComp->SetAutoAttachParams(MeshComp, NAME_None);

		// When recycling we can avoid setting the template if set already.
		if (NewParticleComp->Template != PSTemplate)
		{
			NewParticleComp->SetTemplate(PSTemplate);
		}

		// Recycled components are usually already registered
		if (!NewParticleComp->IsRegistered())
		{
			NewParticleComp->RegisterComponentWithWorld(MeshComp->GetWorld());
		}

		NewParticleComp->AttachToComponent(MeshComp, FAttachmentTransformRules::KeepRelativeTransform);
		NewParticleComp->ActivateSystem(true);

		UParticleSystemComponent::TrailEmitterArray TrailEmitters;
		NewParticleComp->GetOwnedTrailEmitters(TrailEmitters, this, true);

		for (FParticleAnimTrailEmitterInstance* Trail : TrailEmitters)
		{
			Trail->BeginTrail();
			Trail->SetTrailSourceData(FirstSocketName, SecondSocketName, WidthScaleMode, Width);

#if WITH_EDITORONLY_DATA
			Trail->SetTrailDebugData(bRenderGeometry, bRenderSpawnPoints, bRenderTessellation, bRenderTangents);
#endif
		}
	}

	Received_NotifyBegin(MeshComp, Animation, TotalDuration, EventReference);
}

void UAnimNotifyState_Trail::NotifyTick(class USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference)
{
	bool bError = ValidateInput(MeshComp, true);

	if (MeshComp->GetWorld()->GetNetMode() == NM_DedicatedServer)
	{
		return;
	}

	ParticleSystemComponentArray Children;
	GetCandidateSystems(*MeshComp, Children);
	
	float Width = GetCurveWidth(MeshComp);

	for (UParticleSystemComponent* ParticleComp : Children)
	{
		if (ParticleComp->IsActive())
		{
			UParticleSystemComponent::TrailEmitterArray TrailEmitters;
			ParticleComp->GetOwnedTrailEmitters(TrailEmitters, this, false);
			if (bError && TrailEmitters.Num() > 0)
			{
				ParticleComp->DestroyComponent();
			}
			else
			{
				for (FParticleAnimTrailEmitterInstance* Trail : TrailEmitters)
				{
					Trail->SetTrailSourceData(FirstSocketName, SecondSocketName, WidthScaleMode, Width);

	#if WITH_EDITORONLY_DATA
					Trail->SetTrailDebugData(bRenderGeometry, bRenderSpawnPoints, bRenderTessellation, bRenderTangents);
	#endif
				}
			}
		}
	}

	Received_NotifyTick(MeshComp, Animation, FrameDeltaTime, EventReference);
}

void UAnimNotifyState_Trail::NotifyEnd(class USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, const FAnimNotifyEventReference& EventReference)
{
	if (MeshComp->GetWorld()->GetNetMode() == NM_DedicatedServer)
	{
		return;
	}

	ParticleSystemComponentArray Children;
	GetCandidateSystems(*MeshComp, Children);

	for (UParticleSystemComponent* ParticleComp : Children)
	{
		if (ParticleComp->IsActive())
		{
			UParticleSystemComponent::TrailEmitterArray TrailEmitters;
			ParticleComp->GetOwnedTrailEmitters(TrailEmitters, this, false);
			for (FParticleAnimTrailEmitterInstance* Trail : TrailEmitters)
			{
				Trail->EndTrail();
			}
		}
	}

	Received_NotifyEnd(MeshComp, Animation, EventReference);
}

UParticleSystemComponent* UAnimNotifyState_Trail::GetParticleSystemComponent(USkeletalMeshComponent* MeshComp) const
{
	if (MeshComp == nullptr)
	{
		return nullptr;
	}

	ParticleSystemComponentArray Children;
	GetCandidateSystems(*MeshComp, Children);
	for (UParticleSystemComponent* ParticleComp : Children)
	{
		if (ParticleComp->IsActive())
		{
			UParticleSystemComponent::TrailEmitterArray TrailEmitters;
			ParticleComp->GetOwnedTrailEmitters(TrailEmitters, this, false);
			if (TrailEmitters.Num() > 0)
			{
				// We have a trail emitter, so return this one
				return ParticleComp;
			}
		}
	}
	return nullptr;
}

bool UAnimNotifyState_Trail::ValidateInput(class USkeletalMeshComponent * MeshComp, bool bReportErrors/* =false */)
{
#if WITH_EDITOR
	bool bError = false;

	MeshComp->ClearAnimNotifyErrors(this);

	//Validate the user input and report any errors.
	if (FirstSocketName == NAME_None)
	{
		if (bReportErrors)
		{
			const FText FirstSocketEqualsNoneErrorText = FText::Format( LOCTEXT("NoneFirstSocket", "{0}: Must set First Socket Name."), FText::FromString(GetName()));
			MeshComp->ReportAnimNotifyError(FirstSocketEqualsNoneErrorText, this);
		}
		bError = true;
	}

	if (SecondSocketName == NAME_None)
	{
		if (bReportErrors)
		{
			const FText SecondSocketEqualsNoneErrorText = FText::Format( LOCTEXT("NoneSecondSocket", "{0}: Must set Second Socket Name."), FText::FromString(GetName()));
			MeshComp->ReportAnimNotifyError(SecondSocketEqualsNoneErrorText, this);
		}
		bError = true;
	}

	if (!PSTemplate)
	{
		if (bReportErrors)
		{
			const FText PSTemplateEqualsNoneErrorText = FText::Format( LOCTEXT("NonePSTemplate", "{0}: Trail must have a PSTemplate."), FText::FromString(GetName()));
			MeshComp->ReportAnimNotifyError(PSTemplateEqualsNoneErrorText, this);
		}
		bError = true;
	}
	else
	{
		if (!PSTemplate->ContainsEmitterType(UParticleModuleTypeDataAnimTrail::StaticClass()))
		{
			if (bReportErrors)
			{
				const FString PSTemplateName = PSTemplate ? PSTemplate->GetName() : "";
				const FText PSTemplateInvalidErrorText = FText::Format(LOCTEXT("InvalidPSTemplateFmt", "{0}: {1} does not contain any trail emittter."), FText::FromString(GetName()), FText::FromString(PSTemplateName));
				MeshComp->ReportAnimNotifyError(PSTemplateInvalidErrorText, this);
			}

			bError = true;
		}
	}
	return bError;
#else
	return false;
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
