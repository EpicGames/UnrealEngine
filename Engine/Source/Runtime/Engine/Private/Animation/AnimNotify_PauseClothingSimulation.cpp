// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotify_PauseClothingSimulation.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_PauseClothingSimulation)

UAnimNotify_PauseClothingSimulation::UAnimNotify_PauseClothingSimulation()
	: Super()
{
#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(90, 220, 255, 255);
#endif // WITH_EDITORONLY_DATA
}

void UAnimNotify_PauseClothingSimulation::Notify(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	MeshComp->SuspendClothingSimulation();
}

FString UAnimNotify_PauseClothingSimulation::GetNotifyName_Implementation() const
{
	return TEXT("Pause Clothing Sim");
}
