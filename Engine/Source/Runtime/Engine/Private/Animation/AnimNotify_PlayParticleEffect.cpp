// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotify_PlayParticleEffect.h"
#include "Particles/ParticleSystem.h"
#include "Components/SkeletalMeshComponent.h"
#include "ParticleHelper.h"
#include "Kismet/GameplayStatics.h"
#include "Animation/AnimSequenceBase.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_PlayParticleEffect)

#if WITH_EDITOR
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#endif

/////////////////////////////////////////////////////
// UAnimNotify_PlayParticleEffect

UAnimNotify_PlayParticleEffect::UAnimNotify_PlayParticleEffect()
	: Super()
{
	Attached = true;
	Scale = FVector(1.f);

#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(192, 255, 99, 255);
#endif // WITH_EDITORONLY_DATA
}

void UAnimNotify_PlayParticleEffect::PostLoad()
{
	Super::PostLoad();

	RotationOffsetQuat = FQuat(RotationOffset);
}

#if WITH_EDITOR
void UAnimNotify_PlayParticleEffect::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimNotify_PlayParticleEffect, RotationOffset))
	{
		RotationOffsetQuat = FQuat(RotationOffset);
	}
}

void UAnimNotify_PlayParticleEffect::ValidateAssociatedAssets()
{
	static const FName NAME_AssetCheck("AssetCheck");

	if ((PSTemplate != nullptr) && (PSTemplate->IsLooping()))
	{
		UObject* ContainingAsset = GetContainingAsset();
			
		FMessageLog AssetCheckLog(NAME_AssetCheck);

		const FText MessageLooping = FText::Format(
			NSLOCTEXT("AnimNotify", "ParticleSystem_ShouldNotLoop", "Particle system {0} used in anim notify for asset {1} is set to looping, but the slot is a one-shot (it won't be played to avoid leaking a component per notify)."),
			FText::AsCultureInvariant(PSTemplate->GetPathName()),
			FText::AsCultureInvariant(ContainingAsset->GetPathName()));
		AssetCheckLog.Warning()
			->AddToken(FUObjectToken::Create(ContainingAsset))
			->AddToken(FTextToken::Create(MessageLooping));

		if (GIsEditor)
		{
			AssetCheckLog.Notify(MessageLooping, EMessageSeverity::Warning, /*bForce=*/ true);
		}
	}
}
#endif

void UAnimNotify_PlayParticleEffect::Notify(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	// Don't call super to avoid unnecessary call in to blueprints
	SpawnParticleSystem(MeshComp, Animation);
}

FString UAnimNotify_PlayParticleEffect::GetNotifyName_Implementation() const
{
	if (PSTemplate)
	{
		return PSTemplate->GetName();
	}
	else
	{
		return Super::GetNotifyName_Implementation();
	}
}

UParticleSystemComponent* UAnimNotify_PlayParticleEffect::SpawnParticleSystem(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation)
{
	UParticleSystemComponent* ReturnComp = nullptr;

	if (PSTemplate)
	{
		if (PSTemplate->IsLooping())
		{
			UE_LOGF(LogParticles, Warning, "Particle Notify: Anim '%ls' tried to spawn infinitely looping particle system '%ls'. Spawning suppressed.", *GetNameSafe(Animation), *GetNameSafe(PSTemplate));
			return ReturnComp;
		}

		if (Attached)
		{
			ReturnComp = UGameplayStatics::SpawnEmitterAttached(PSTemplate, MeshComp, SocketName, LocationOffset, RotationOffset, Scale);
		}
		else
		{
			const FTransform MeshTransform = MeshComp->GetSocketTransform(SocketName);
			FTransform SpawnTransform;
			SpawnTransform.SetLocation(MeshTransform.TransformPosition(LocationOffset));
			SpawnTransform.SetRotation(MeshTransform.GetRotation() * RotationOffsetQuat);
			SpawnTransform.SetScale3D(Scale);
			ReturnComp = UGameplayStatics::SpawnEmitterAtLocation(MeshComp->GetWorld(), PSTemplate, SpawnTransform);
		}
	}
	else
	{
		UE_LOGF(LogParticles, Warning, "Particle Notify: Particle system is null for particle notify '%ls' in anim: '%ls'", *GetNotifyName(), *GetPathNameSafe(Animation));
	}

	return ReturnComp;
}
