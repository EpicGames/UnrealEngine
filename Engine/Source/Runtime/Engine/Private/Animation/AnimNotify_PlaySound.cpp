// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotify_PlaySound.h"
#include "Audio.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"
#include "Sound/SoundBase.h"
#include "Animation/AnimSequenceBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_PlaySound)

#if WITH_EDITOR
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#endif

/////////////////////////////////////////////////////
// UAnimNotify_PlaySound

UAnimNotify_PlaySound::UAnimNotify_PlaySound()
	: Super()
{
	VolumeMultiplier = 1.f;
	PitchMultiplier = 1.f;

#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(196, 142, 255, 255);
	bFollowEditable = true;
	bPreviewIgnoreAttenuation = false;
#endif // WITH_EDITORONLY_DATA
}

void UAnimNotify_PlaySound::Notify(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	// Don't call super to avoid call back in to blueprints
	if (Sound && MeshComp)
	{
		if (!Sound->IsOneShot())
		{
			UE_LOGF(LogAudio, Warning, "PlaySound notify: Anim %ls tried to play a sound asset which is not a one-shot: '%ls'. Spawning suppressed.", *GetNameSafe(Animation), *GetNameSafe(Sound));
			return;
		}

#if WITH_EDITORONLY_DATA
		UWorld* World = MeshComp->GetWorld();
		if (bPreviewIgnoreAttenuation && World && World->WorldType == EWorldType::EditorPreview)
		{
			if (MeshComp->IsPlaying())
			{
				UGameplayStatics::PlaySound2D(World, Sound, VolumeMultiplier, PitchMultiplier);
			}
		}
		else
#endif
		{
			if (bFollow)
			{
				UGameplayStatics::SpawnSoundAttached(Sound, MeshComp, AttachName, FVector(ForceInit), EAttachLocation::SnapToTarget, false, VolumeMultiplier, PitchMultiplier);
			}
			else
			{
				UGameplayStatics::PlaySoundAtLocation(MeshComp->GetWorld(), Sound, MeshComp->GetComponentLocation(), VolumeMultiplier, PitchMultiplier);
			}
		}
	}
}

FString UAnimNotify_PlaySound::GetNotifyName_Implementation() const
{
	if (Sound)
	{
		return Sound->GetName();
	}
	else
	{
		return Super::GetNotifyName_Implementation();
	}
}

#if WITH_EDITOR
void UAnimNotify_PlaySound::ValidateAssociatedAssets()
{
	static const FName NAME_AssetCheck("AssetCheck");

	if (Sound != nullptr && !Sound->IsOneShot())
	{
		UObject* ContainingAsset = GetContainingAsset();

		FMessageLog AssetCheckLog(NAME_AssetCheck);

		const FText MessageLooping = FText::Format(
			NSLOCTEXT("AnimNotify", "Sound_ShouldNotLoop", "Sound {0} used in anim notify for asset {1} is set to looping, but the slot is a one-shot (it won't be played to avoid leaking an instance per notify)."),
			FText::AsCultureInvariant(Sound->GetPathName()),
			FText::AsCultureInvariant(ContainingAsset->GetPathName()));
		AssetCheckLog.Warning()
			->AddToken(FUObjectToken::Create(ContainingAsset))
			->AddToken(FTextToken::Create(MessageLooping));

		if (GIsEditor)
		{
			AssetCheckLog.Notify(MessageLooping, EMessageSeverity::Warning, /*bForce=*/ true);
		}
	}
}

void UAnimNotify_PlaySound::PostLoad()
{
	Super::PostLoad();
	
	if (!AttachName.IsNone())
	{
		// Sound MUST follow its owner if it follows one of its bones (edit condition prevents bFollow to become false in this case).
		bFollowEditable = false;
		bFollow = true;
	}
	else
	{
		// If no bone name is given, the user can choose whether it follows the root bone or not by setting bFollow themselves.
		bFollowEditable = true;
	}
}

void UAnimNotify_PlaySound::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, AttachName))
	{
		if (!AttachName.IsNone())
		{
			// Sound MUST follow its owner if it follows one of its bones (edit condition prevents bFollow to become false in this case).
			bFollowEditable = false;
			bFollow = true;
		}
		else
		{
			// If no bone name is given, the user can choose whether it follows the root bone or not by setting bFollow themselves.
			bFollowEditable = true;
		}
	}
}
#endif
