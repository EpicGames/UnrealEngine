// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotify_ResetClothingSimulation.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_ResetClothingSimulation)

UAnimNotify_ResetClothingSimulation::UAnimNotify_ResetClothingSimulation()
	: Super()
{
#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(90, 220, 255, 255);
#endif // WITH_EDITORONLY_DATA
}

void UAnimNotify_ResetClothingSimulation::Notify(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	MeshComp->ForceClothNextUpdateTeleportAndReset();
}

FString UAnimNotify_ResetClothingSimulation::GetNotifyName_Implementation() const
{
	return TEXT("Reset Clothing Sim");
}
