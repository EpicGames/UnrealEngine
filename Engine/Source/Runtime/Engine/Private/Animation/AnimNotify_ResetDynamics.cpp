// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotify_ResetDynamics.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_ResetDynamics)

UAnimNotify_ResetDynamics::UAnimNotify_ResetDynamics()
	: Super()
{
#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(90, 220, 255, 255);
#endif // WITH_EDITORONLY_DATA
}

void UAnimNotify_ResetDynamics::Notify(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	MeshComp->ResetAnimInstanceDynamics();
}

FString UAnimNotify_ResetDynamics::GetNotifyName_Implementation() const
{
	return TEXT("Reset Dynamics");
}
