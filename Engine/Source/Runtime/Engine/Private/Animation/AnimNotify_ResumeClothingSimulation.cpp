// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimNotifies/AnimNotify_ResumeClothingSimulation.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_ResumeClothingSimulation)

UAnimNotify_ResumeClothingSimulation::UAnimNotify_ResumeClothingSimulation()
	: Super()
{
#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(90, 220, 255, 255);
#endif // WITH_EDITORONLY_DATA
}

void UAnimNotify_ResumeClothingSimulation::Notify(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	MeshComp->ResumeClothingSimulation();
}

FString UAnimNotify_ResumeClothingSimulation::GetNotifyName_Implementation() const
{
	return TEXT("Resume Clothing Sim");
}
