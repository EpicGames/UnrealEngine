// Copyright Epic Games, Inc. All Rights Reserved.

// This code is based upon and adapted to Unreal from the code 
// provided in the Sandbox project here:
// https://github.com/melax/sandbox

// The license for Sandbox is reproduced below:

/*
Copyright (c) 2014 Stan Melax

Permission is hereby granted, free of charge, to any person obtaining a copy
of this software and associated documentation files (the "Software"), to deal
in the Software without restriction, including without limitation the rights
to use, copy, modify, merge, publish, distribute, sublicense, and/or sell
copies of the Software, and to permit persons to whom the Software is
furnished to do so, subject to the following conditions:

The above copyright notice and this permission notice shall be included in all
copies or substantial portions of the Software.

THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE
SOFTWARE.
*/

#include "Animation/AnimPhysicsSolver.h"
#include "HAL/IConsoleManager.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Stats/Stats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimPhysicsSolver)

DEFINE_STAT(STAT_AnimDynamicsUpdate);
DEFINE_STAT(STAT_AnimDynamicsLinearPre);
DEFINE_STAT(STAT_AnimDynamicsLinearPost);
DEFINE_STAT(STAT_AnimDynamicsAngularPre);
DEFINE_STAT(STAT_AnimDynamicsAngularPost);
DEFINE_STAT(STAT_AnimDynamicsVelocityInit);
DEFINE_STAT(STAT_AnimDynamicsPoseUpdate);
DEFINE_STAT(STAT_AnimDynamicsLimitUpdate);

bool FAnimPhys::bEnableDetailedStats = false;
static FAutoConsoleVariableRef CVarAnimDynamicsDetailedStats(TEXT("p.AnimDynamicsDetailedStats"), FAnimPhys::bEnableDetailedStats, TEXT("When set to 1, will enable more detailed stats."));

float FAnimPhys::CentrifugalAlpha = 1.0f;
static FAutoConsoleVariableRef CVarAnimDynamicsSimSpaceCentrifugalAlpha(TEXT("p.AnimDynamics.SimSpaceCentrifugalAlpha"), FAnimPhys::CentrifugalAlpha, TEXT("Settings for centrifugal force alpha for anim dynamics nodes"));

float FAnimPhys::CoriolisAlpha = 0.5f;
static FAutoConsoleVariableRef CVarAnimDynamicsSimSpaceCoriolisAlpha(TEXT("p.AnimDynamics.SimSpaceCoriolisAlpha"), FAnimPhys::CoriolisAlpha, TEXT("Settings for centrifugal force alpha for anim dynamics nodes"));

float FAnimPhys::EulerAlpha = 1.0f;
static FAutoConsoleVariableRef CVarAnimDynamicsSimSpaceEulerAlpha(TEXT("p.AnimDynamics.SimSpaceEulerAlpha"), FAnimPhys::EulerAlpha, TEXT("Settings for euler force alpha for anim dynamics nodes"));

//////////////////////////////////////////////////////////////////////////
// FAnimPhysShape - Defines a set of vertices that make a shape with volume and CoM

FAnimPhysShape::FAnimPhysShape() : Volume(0.0f)
, CenterOfMass(0.0f)
{

}

FAnimPhysShape::FAnimPhysShape(TArray<FVector>& InVertices, TArray<FIntVector>& InTriangles) : Vertices(InVertices)
, Triangles(InTriangles)
{
	Volume = FAnimPhys::CalculateVolume(Vertices, Triangles);
	CenterOfMass = FAnimPhys::CalculateCenterOfMass(Vertices, Triangles);
}

FAnimPhysShape FAnimPhysShape::MakeBox(const FVector& Extents)
{
	FVector HalfExtents = Extents / 2.0f;

	// A box of zero size will introduce NaNs into the simulation so we stomp it here and log
	// if we encounter it.
	if (Extents.SizeSquared() <= UE_SMALL_NUMBER)
	{
// 		UE_LOGF(LogAnimation, Warning, "AnimDynamics: Attempted to create a simulation box with 0 volume, this introduces NaNs into the simulation. Adjusting box extents to (1.0f,1.0f,1.0f)");
		HalfExtents = FVector(0.5f);
	}

	TArray<FVector> Verts;
	TArray<FIntVector> Tris;
	Verts.Reserve(8);
	Tris.Reserve(12);	

	// Front Verts
	Verts.Add(FVector(-HalfExtents.X, -HalfExtents.Y, HalfExtents.Z));
	Verts.Add(FVector(HalfExtents.X, -HalfExtents.Y, HalfExtents.Z));
	Verts.Add(FVector(HalfExtents.X, -HalfExtents.Y, -HalfExtents.Z));
	Verts.Add(FVector(-HalfExtents.X, -HalfExtents.Y, -HalfExtents.Z));

	// Back Verts
	Verts.Add(FVector(HalfExtents.X, HalfExtents.Y, HalfExtents.Z));
	Verts.Add(FVector(-HalfExtents.X, HalfExtents.Y, HalfExtents.Z));
	Verts.Add(FVector(-HalfExtents.X, HalfExtents.Y, -HalfExtents.Z));
	Verts.Add(FVector(HalfExtents.X, HalfExtents.Y, -HalfExtents.Z));

	// Front
	Tris.Add(FIntVector(0, 1, 3));
	Tris.Add(FIntVector(1, 2, 3));

	// Back
	Tris.Add(FIntVector(4, 5, 7));
	Tris.Add(FIntVector(5, 6, 7));

	// Top
	Tris.Add(FIntVector(0, 5, 1));
	Tris.Add(FIntVector(5, 4, 1));

	// Right
	Tris.Add(FIntVector(1, 4, 2));
	Tris.Add(FIntVector(2, 4, 7));

	// Left
	Tris.Add(FIntVector(0, 3, 5));
	Tris.Add(FIntVector(5, 3, 6));

	// Bottom
	Tris.Add(FIntVector(3, 2, 6));
	Tris.Add(FIntVector(2, 7, 6));

	return FAnimPhysShape(Verts, Tris);
}

void FAnimPhysShape::TransformVerts(FTransform& InTransform)
{
	for(FVector& Vert : Vertices)
	{
		Vert = InTransform.TransformVector(Vert);
	}
}

//////////////////////////////////////////////////////////////////////////
// FAnimPhysState - Pose with momentum

FAnimPhysState::FAnimPhysState() : LinearMomentum(0, 0, 0)
, AngularMomentum(0, 0, 0)
{

}

FAnimPhysState::FAnimPhysState(const FVector& InPosition, const FQuat& InOrient, const FVector& InLinearMomentum, const FVector& InAngularMomentum) 
: Pose(InPosition, InOrient)
, LinearMomentum(InLinearMomentum)
, AngularMomentum(InAngularMomentum)
{

}

//////////////////////////////////////////////////////////////////////////

FAnimPhysRigidBody::FAnimPhysRigidBody(TArray<FAnimPhysShape>& InShapes, const FVector& InPosition) : Mass(1.0f)
, InverseMass(1.0f)
, InverseTensorWithoutMass(FMatrix::Identity)
, InverseWorldSpaceTensor(FMatrix::Identity)
, NextOrientation(FQuat::Identity)
, PreviousOrientation(FQuat::Identity)
, StartOrientation(FQuat::Identity)
, bAngularDampingOverriden(false)
, AngularDamping(0.0f)
, bLinearDampingOverriden(false)
, LinearDamping(0.0f)
, GravityScale(1.0f)
, bUseGravityOverride(false)
, GravityOverride(FVector::ZeroVector)
, Shapes(InShapes)
{
	StartPosition = InPosition;
	PreviousPosition = InPosition;
	NextPosition = InPosition;
	Pose.Position = InPosition;

	CenterOfMass = FAnimPhys::CalculateCenterOfMass(Shapes);
	Pose.Position += CenterOfMass;
	StartPosition = Pose.Position;
	PreviousPosition = Pose.Position;
	NextPosition = Pose.Position;

	for (FAnimPhysShape& CurrentShape : Shapes)
	{
		for (FVector& CurrentVert : CurrentShape.Vertices)
		{
			CurrentVert -= CenterOfMass;
		}
	}

	InertiaTensor = FAnimPhys::CalculateInertia(Shapes, CenterOfMass);
	InverseMass = 1.0f / Mass;
	InverseTensorWithoutMass = InertiaTensor.InverseFast();
	InverseWorldSpaceTensor = InverseTensorWithoutMass * InverseMass;
}

FVector FAnimPhysRigidBody::Spin()
{

	return InverseWorldSpaceTensor.TransformVector(AngularMomentum);
}

//////////////////////////////////////////////////////////////////////////

FAnimPhysLimit::FAnimPhysLimit(FAnimPhysRigidBody* InFirstBody, FAnimPhysRigidBody* InSecondBody)
{
	Bodies[0] = InFirstBody;
	Bodies[1] = InSecondBody;
}

//////////////////////////////////////////////////////////////////////////

FAnimPhysAngularLimit::FAnimPhysAngularLimit() :FAnimPhysLimit(nullptr, nullptr)
{

}

FAnimPhysAngularLimit::FAnimPhysAngularLimit(FAnimPhysRigidBody* InFirstBody, FAnimPhysRigidBody* InSecondBody, const FVector& InWorldSpaceAxis, float InTargetSpin /*= 0*/, float InMinimumTorque /*= -MAX_flt*/, float InMaximumTorque /*= MAX_flt*/) 
: FAnimPhysLimit(InFirstBody, InSecondBody)
, WorldSpaceAxis(InWorldSpaceAxis)
, Torque(0.0f)
, TargetSpin(InTargetSpin)
, MinimumTorque(InMinimumTorque)
, MaximumTorque(InMaximumTorque)
{
	UpdateCachedData();
}

void FAnimPhysAngularLimit::RemoveBias()
{
	// not zero since its ok to let one-sided constraints fall to their bound;
	TargetSpin = (MinimumTorque < 0) ? 0 : FMath::Min(TargetSpin, 0.0f);
}

void FAnimPhysAngularLimit::Iter(float DeltaTime)
{
	if (TargetSpin == -MAX_flt)
	{
		return;
	}

	// how we are rotating about the axis 'normal' we are dealing with
	float CurrentSpin = ((Bodies[1]) ? FVector::DotProduct(Bodies[1]->Spin(), WorldSpaceAxis) : 0.0f) - ((Bodies[0]) ? FVector::DotProduct(Bodies[0]->Spin(), WorldSpaceAxis) : 0.0f);

	// the amount of spin we have to add to satisfy the limit.
	float DeltaSpin = TargetSpin - CurrentSpin;

	// how we have to change the angular impulse
	float DeltaTorque = DeltaSpin * CachedSpinToTorque;

	// total torque cannot exceed maxtorque
	DeltaTorque = FMath::Min(DeltaTorque, MaximumTorque * DeltaTime - Torque);
	// total torque cannot fall below mintorque
	DeltaTorque = FMath::Max(DeltaTorque, MinimumTorque * DeltaTime - Torque);

	// Apply impulses
	if (Bodies[0])
	{
		Bodies[0]->AngularMomentum -= WorldSpaceAxis * DeltaTorque;
	}
	if (Bodies[1])
	{
		Bodies[1]->AngularMomentum += WorldSpaceAxis * DeltaTorque;
	}

	Torque += DeltaTorque;
}

void FAnimPhysAngularLimit::UpdateCachedData()
{
	CachedSpinToTorque = 1.0f / (((Bodies[0]) ? FVector::DotProduct(WorldSpaceAxis, Bodies[0]->InverseWorldSpaceTensor.TransformVector(WorldSpaceAxis)) : 0.0f) + ((Bodies[1]) ? FVector::DotProduct(WorldSpaceAxis, Bodies[1]->InverseWorldSpaceTensor.TransformVector(WorldSpaceAxis)) : 0.0f));
}

//////////////////////////////////////////////////////////////////////////

FAnimPhysLinearLimit::FAnimPhysLinearLimit() 
: FAnimPhysLimit(nullptr, nullptr)
{

}

FAnimPhysLinearLimit::FAnimPhysLinearLimit(FAnimPhysRigidBody* InFirstBody, FAnimPhysRigidBody* InSecondBody, const FVector& InFirstPosition, const FVector& InSecondPosition, const FVector& InNormal /*= FVector(0.0f, 0.0f, 1.0f)*/, float InTargetSpeed /*= 0.0f*/, float InTargetSpeedWithoutBias /*= 0.0f*/, const FVector2D& InForceRange /*= FVector2D(-MAX_flt, MAX_flt)*/) 
: FAnimPhysLimit(InFirstBody, InSecondBody)
, FirstPosition(InFirstPosition)
, SecondPosition(InSecondPosition)
, LimitNormal(InNormal)
, TargetSpeed(InTargetSpeed)
, TargetSpeedWithoutBias(InTargetSpeedWithoutBias)
, MinimumForce(FMath::Min(InForceRange.X, InForceRange.Y))
, Maximumforce(FMath::Max(InForceRange.X, InForceRange.Y))
, SumImpulses(0.0f)
{
	UpdateCachedData();
}

void FAnimPhysLinearLimit::RemoveBias()
{
	TargetSpeed = TargetSpeedWithoutBias;
}

void FAnimPhysLinearLimit::Iter(float DeltaTime)
{
	// Instantaneous linear velocity at point of constraint
	FVector Velocity0 = Bodies[0] ? FVector::CrossProduct(Bodies[0]->Spin(), WorldSpacePosition0) + Bodies[0]->LinearMomentum * Bodies[0]->InverseMass : FVector(0.0f, 0.0f, 0.0f);
	FVector Velocity1 = Bodies[1] ? FVector::CrossProduct(Bodies[1]->Spin(), WorldSpacePosition1) + Bodies[1]->LinearMomentum * Bodies[1]->InverseMass : FVector(0.0f, 0.0f, 0.0f);

	// velocity of rb1 wrt rb0
	float  DeltaVelocity = FVector::DotProduct(Velocity1 - Velocity0, LimitNormal);

	float VelocityImpulse = -TargetSpeed - DeltaVelocity;

	float ResultantImpulse = VelocityImpulse * InverseInertiaImpulse;

	ResultantImpulse = FMath::Min(Maximumforce * DeltaTime - SumImpulses, ResultantImpulse);
	ResultantImpulse = FMath::Max(MinimumForce * DeltaTime - SumImpulses, ResultantImpulse);

	if (Bodies[0])
	{
		FAnimPhys::ApplyImpulse(Bodies[0], WorldSpacePosition0, LimitNormal * -ResultantImpulse);
	}

	if (Bodies[1])
	{
		FAnimPhys::ApplyImpulse(Bodies[1], WorldSpacePosition1, LimitNormal * ResultantImpulse);
	}

	SumImpulses += ResultantImpulse;
}
void FAnimPhysLinearLimit::UpdateCachedData()
{
	WorldSpacePosition0 = (Bodies[0]) ? Bodies[0]->Pose.Orientation * FirstPosition : FirstPosition;
	WorldSpacePosition1 = (Bodies[1]) ? Bodies[1]->Pose.Orientation * SecondPosition : SecondPosition;
	InverseInertiaImpulse = 1.0f / ((Bodies[0] ? Bodies[0]->InverseMass + FVector::DotProduct(FVector::CrossProduct(Bodies[0]->InverseWorldSpaceTensor.TransformVector(FVector::CrossProduct(WorldSpacePosition0, LimitNormal)), WorldSpacePosition0), LimitNormal) : 0)
		+ (Bodies[1] ? Bodies[1]->InverseMass + FVector::DotProduct(FVector::CrossProduct(Bodies[1]->InverseWorldSpaceTensor.TransformVector(FVector::CrossProduct(WorldSpacePosition1, LimitNormal)), WorldSpacePosition1), LimitNormal) : 0));
}

//////////////////////////////////////////////////////////////////////////

float FAnimPhys::CalculateVolume(const TArray<FVector>& InVertices, const TArray<FIntVector>& InTriangles)
{
	float CalculatedVolume = 0;

	for (const FIntVector& Tri : InTriangles)
	{
		FMatrix VolMatrix(InVertices[Tri[0]], InVertices[Tri[1]], InVertices[Tri[2]], FVector(1.0f));

		// This gives us six times the volume of the tetrahedron, we'll divide through later
		CalculatedVolume += VolMatrix.RotDeterminant();
	}

	return CalculatedVolume / 6.0f;
}

float FAnimPhys::CalculateVolume(const TArray<FAnimPhysShape>& InShapes)
{
	float TotalVolume = 0.0f;
	for (const FAnimPhysShape& CurrentShape : InShapes)
	{
		TotalVolume += FAnimPhys::CalculateVolume(CurrentShape.Vertices, CurrentShape.Triangles);
	}

	return TotalVolume;
}

FVector FAnimPhys::CalculateCenterOfMass(const TArray<FVector>& InVertices, const TArray<FIntVector>& InTriangles)
{
	FVector CoM(0.0f);
	float CalculatedVolume = 0.0f;
	FVector Vertices[3];

	for (const FIntVector& Tri : InTriangles)
	{
		Vertices[0] = InVertices[Tri[0]];
		Vertices[1] = InVertices[Tri[1]];
		Vertices[2] = InVertices[Tri[2]];
		FMatrix VolMatrix(Vertices[0], Vertices[1], Vertices[2], FVector(1.0f));

		// Six times volume, doesn't need to be divided for CoM (Balanced)
		float TetraVolume = VolMatrix.RotDeterminant();

		// Avg, divide by 4 at the end
		CoM += TetraVolume * (Vertices[0] + Vertices[1] + Vertices[2]);

		CalculatedVolume += TetraVolume;
	}

	CoM /= CalculatedVolume * 4.0f;
	return CoM;
}

FVector FAnimPhys::CalculateCenterOfMass(const TArray<FAnimPhysShape>& InShapes)
{
	FVector TotalCenterOfMass(0.0f);
	float TotalVolume = 0.0f;

	for (const FAnimPhysShape& CurrentShape : InShapes)
	{
		FVector ShapeCenterOfMass = FAnimPhys::CalculateCenterOfMass(CurrentShape.Vertices, CurrentShape.Triangles);
		float ShapeVolume = FAnimPhys::CalculateVolume(CurrentShape.Vertices, CurrentShape.Triangles);

		TotalVolume += ShapeVolume;
		TotalCenterOfMass += ShapeCenterOfMass * ShapeVolume;
	}

	TotalCenterOfMass /= TotalVolume;
	return TotalCenterOfMass;
}

FMatrix FAnimPhys::CalculateInertia(const TArray<FVector>& InVertices, const TArray<FIntVector>& InTriangles, const FVector& InCenterOfMass)
{
	// Calculate moments around CoM (FVector::ZeroVector if unsupplied)
	// Mass is assumed to be 1.0f, tensor can be scaled later

	// Accumulates volume time six
	float CalculatedVolume = 0.0f;
	// Main diagonal integrals
	FVector TensorDiagonals(0.0f);
	// Off-Diagonal integrals
	FVector TensorOffDiagonals(0.0f);

	FVector Vertices[3];

	for (const FIntVector& Tri : InTriangles)
	{
		Vertices[0] = InVertices[Tri[0]];
		Vertices[1] = InVertices[Tri[1]];
		Vertices[2] = InVertices[Tri[2]];

		// Calculate tetrahedron volume (times 6)
		FMatrix VolMatrix(Vertices[0], Vertices[1], Vertices[2], FVector(1.0f));
		float TetraVolume = VolMatrix.RotDeterminant();
		CalculatedVolume += TetraVolume;

		for (int32 V0 = 0; V0 < 3; ++V0)
		{
			int32 V1 = (V0 + 1) % 3;
			int32 V2 = (V0 + 2) % 3;

			// Calculate diagonals times 60 (needs divide later)
			TensorDiagonals[V0] += (Vertices[0][V0] * Vertices[1][V0] + Vertices[1][V0] * Vertices[2][V0] + Vertices[2][V0] * Vertices[0][V0] +
				Vertices[0][V0] * Vertices[0][V0] + Vertices[1][V0] * Vertices[1][V0] + Vertices[2][V0] * Vertices[2][V0]) * TetraVolume;

			// Calculate off-diagonals times 120 (needs divide later)
			TensorOffDiagonals[V0] += (Vertices[0][V1] * Vertices[1][V2] + Vertices[1][V1] * Vertices[2][V2] + Vertices[2][V1] * Vertices[0][V2] +
				Vertices[0][V1] * Vertices[2][V2] + Vertices[1][V1] * Vertices[0][V2] + Vertices[2][V1] * Vertices[1][V2] +
				Vertices[0][V1] * Vertices[0][V2] * 2 + Vertices[1][V1] * Vertices[1][V2] * 2 + Vertices[2][V1] * Vertices[2][V2] * 2) * TetraVolume;
		}
	}

	// Divide through by required amounts (and the six for the volume calculation)
	TensorDiagonals /= CalculatedVolume * (60.0f / 6.0f);
	TensorOffDiagonals /= CalculatedVolume * (120.0f / 6.0f);

	return FMatrix(FVector(TensorDiagonals.Y + TensorDiagonals.Z, -TensorOffDiagonals.Z, -TensorOffDiagonals.Y),
		FVector(-TensorOffDiagonals.Z, TensorDiagonals.X + TensorDiagonals.Z, -TensorOffDiagonals.X),
		FVector(-TensorOffDiagonals.Y, -TensorOffDiagonals.X, TensorDiagonals.X + TensorDiagonals.Y),
		FVector(0.0f));
}

FMatrix FAnimPhys::CalculateInertia(const TArray<FAnimPhysShape>& InShapes, const FVector& InCenterOfMass)
{
	float TotalVolume = 0.0f;
	FMatrix TotalInertia(ForceInitToZero);

	for (const FAnimPhysShape& CurrentShape : InShapes)
	{
		float ShapeVolume = FAnimPhys::CalculateVolume(CurrentShape.Vertices, CurrentShape.Triangles);
		TotalInertia += FAnimPhys::CalculateInertia(CurrentShape.Vertices, CurrentShape.Triangles, InCenterOfMass) * ShapeVolume;
		TotalVolume += ShapeVolume;
	}

	TotalInertia *= 1.0f / TotalVolume;

	// We use 4x4 matrices right now, make sure it's homogeneous for calculations
	TotalInertia.M[3][3] = 1.0f;

	return TotalInertia;
}

void FAnimPhys::ScaleRigidBodyMass(FAnimPhysRigidBody* InOutRigidBody, float Scale) // scales the mass and all relevant inertial properties by multiplier s
{
	InOutRigidBody->Mass *= Scale;
	InOutRigidBody->LinearMomentum *= Scale;
	InOutRigidBody->InverseMass *= 1.0f / Scale;
	InOutRigidBody->AngularMomentum *= Scale;
	InOutRigidBody->InverseWorldSpaceTensor *= 1.0f / Scale;
	InOutRigidBody->InertiaTensor *= Scale;
}

void FAnimPhys::RigidBodyShapeScaleUpdate(FAnimPhysRigidBody* InOutRigidBody, float Scale)
{
	const float InertialScaleFactor = Scale * Scale;

	InOutRigidBody->AngularMomentum *= InertialScaleFactor;
	InOutRigidBody->InverseWorldSpaceTensor *= 1.0f / InertialScaleFactor;
	InOutRigidBody->InertiaTensor *= InertialScaleFactor;
	InOutRigidBody->InverseTensorWithoutMass *= 1.0f / InertialScaleFactor;
}

FQuat FAnimPhys::DiffQ(const FQuat& InOrientation, const FMatrix &InInverseTensor, const FVector& InAngularMomentum)
{
	FQuat NormalisedOrient = InOrientation.GetNormalized();
	FMatrix OrientAsMatrix = FQuatRotationMatrix::Make(NormalisedOrient);
	FMatrix AppliedTensorMatrix = OrientAsMatrix * InInverseTensor * OrientAsMatrix.GetTransposed();
	FVector HalfSpin = AppliedTensorMatrix.TransformVector(InAngularMomentum) * 0.5f;

	FQuat SpinQuat(HalfSpin.X, HalfSpin.Y, HalfSpin.Z, 0.0f);
	return SpinQuat * NormalisedOrient;
}

FQuat FAnimPhys::UpdateOrientRK(const FQuat& InOrient, const FMatrix& InInverseTensor, const FVector& InAngularMomentum, float InDeltaTime)
{
	// RK update for orientation, preserves realistic spin
	FQuat d1 = DiffQ(InOrient, InInverseTensor, InAngularMomentum);
	FQuat d2 = DiffQ(InOrient + d1*(InDeltaTime / 2), InInverseTensor, InAngularMomentum);
	FQuat d3 = DiffQ(InOrient + d2*(InDeltaTime / 2), InInverseTensor, InAngularMomentum);
	FQuat d4 = DiffQ(InOrient + d3*(InDeltaTime), InInverseTensor, InAngularMomentum);

	return (InOrient + d1* (InDeltaTime / 6) + d2 * (InDeltaTime / 3) + d3 * (InDeltaTime / 3) + d4 * (InDeltaTime / 6)).GetNormalized();
}

void FAnimPhys::ApplyImpulse(FAnimPhysRigidBody* InOutRigidBody, const FVector& InWorldOrientedImpactPoint, const FVector& InImpulse)
{
	// InWorldOrientedImpactPoint is impact point positionally relative to InOutRigidBody's origin but in 'world' orientation
	InOutRigidBody->LinearMomentum += InImpulse;
	InOutRigidBody->AngularMomentum += FVector::CrossProduct(InWorldOrientedImpactPoint, InImpulse);
}

void FAnimPhys::ConstrainAlongDirection(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody *FirstBody, const FVector& FirstPosition, FAnimPhysRigidBody *SecondBody, const FVector& SecondPosition, const FVector& AxisToConstrain, const FVector2D Limits, float MinimumForce /*= -MAX_flt*/, float MaximumForce /*= MAX_flt*/)
{
	FVector Position0 = (FirstBody) ? FirstBody->GetPose() * FirstPosition : FirstPosition;
	FVector Position1 = (SecondBody) ? SecondBody->GetPose() * SecondPosition : SecondPosition;

	float Distance = FVector::DotProduct(Position1 - Position0, AxisToConstrain);

	if (FMath::Abs(Limits.X - Limits.Y) < UE_SMALL_NUMBER)
	{
		// Fully locked axis, just generate one limit
		LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, AxisToConstrain, Distance / DeltaTime, Distance / DeltaTime, FVector2D(MinimumForce, MaximumForce)));
	}
	else
	{
		float TargetSpeed0 = (Distance - Limits.X) / DeltaTime;
		float TargetSpeed1 = (Distance - Limits.Y) / DeltaTime;

		LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, AxisToConstrain, TargetSpeed0, TargetSpeed0, FVector2D(0.0f, MaximumForce)));
		LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, AxisToConstrain, TargetSpeed1, TargetSpeed1, FVector2D(MinimumForce, 0.0f)));
	}
}

void FAnimPhys::ConstrainPositionNailed(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody *FirstBody, const FVector& FirstPosition, FAnimPhysRigidBody *SecondBody, const FVector& SecondPosition)
{
	FVector Position0 = (FirstBody) ? FirstBody->GetPose() * FirstPosition : FirstPosition;
	FVector Position1 = (SecondBody) ? SecondBody->GetPose() * SecondPosition : SecondPosition;
	FVector TargetAxisSpeeds = (Position1 - Position0) / DeltaTime;

	LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, FVector(1.0f, 0.0f, 0.0f), TargetAxisSpeeds.X));
	LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, FVector(0.0f, 1.0f, 0.0f), TargetAxisSpeeds.Y));
	LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, FVector(0.0f, 0.0f, 1.0f), TargetAxisSpeeds.Z));
}

void FAnimPhys::ConstrainPositionPrismatic(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody* FirstBody, const FVector& FirstPosition, FAnimPhysRigidBody* SecondBody, const FVector& SecondPosition, const FQuat& PrismRotation, const FVector& LimitsMin, const FVector& LimitsMax)
{
	FVector Position0 = (FirstBody) ? FirstBody->GetPose() * FirstPosition : FirstPosition;
	FVector Position1 = (SecondBody) ? SecondBody->GetPose() * SecondPosition : SecondPosition;
	
	FVector ToPoint = Position1 - Position0;
	
	FVector XAxis = PrismRotation.RotateVector(FVector(1.0f, 0.0f, 0.0f));
	FVector YAxis = PrismRotation.RotateVector(FVector(0.0f, 1.0f, 0.0f));
	FVector ZAxis = PrismRotation.RotateVector(FVector(0.0f, 0.0f, 1.0f));

	FVector AxisDistances(FVector::DotProduct(ToPoint, XAxis), FVector::DotProduct(ToPoint, YAxis), FVector::DotProduct(ToPoint, ZAxis));

	// Target in prism space
	FVector Target(PrismRotation.Inverse().RotateVector(ToPoint));

	// Get closest valid point within the limit
	if (AxisDistances.X < LimitsMin.X)
	{
		Target.X = LimitsMin.X;
	}
	else if (AxisDistances.X > LimitsMax.X)
	{
		Target.X = LimitsMax.X;
	}

	if (AxisDistances.Y < LimitsMin.Y)
	{
		Target.Y = LimitsMin.Y;
	}
	else if (AxisDistances.Y > LimitsMax.Y)
	{
		Target.Y = LimitsMax.Y;
	}

	if (AxisDistances.Z < LimitsMin.Z)
	{
		Target.Z = LimitsMin.Z;
	}
	else if (AxisDistances.Z > LimitsMax.Z)
	{
		Target.Z = LimitsMax.Z;
	}

	// Target in world space
	Target = Position0 + PrismRotation.RotateVector(Target);

	if (!Target.Equals(Position1))
	{
		FVector TargetAxisSpeeds = (Position1 - Target) / DeltaTime;

		if (FMath::Abs(TargetAxisSpeeds.X) > UE_SMALL_NUMBER)
		{
			LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, FVector(1.0f, 0.0f, 0.0f), TargetAxisSpeeds.X));
		}

		if (FMath::Abs(TargetAxisSpeeds.Y) > UE_SMALL_NUMBER)
		{
			LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, FVector(0.0f, 1.0f, 0.0f), TargetAxisSpeeds.Y));
		}

		if (FMath::Abs(TargetAxisSpeeds.Z) > UE_SMALL_NUMBER)
		{
			LimitContainer.Add(FAnimPhysLinearLimit(FirstBody, SecondBody, FirstPosition, SecondPosition, FVector(0.0f, 0.0f, 1.0f), TargetAxisSpeeds.Z));
		}
	}
}

void FAnimPhys::ConstrainAngularRangeInternal(float DeltaTime, TArray<FAnimPhysAngularLimit>& LimitContainer, FAnimPhysRigidBody *FirstBody, const FQuat& JointFrame0, FAnimPhysRigidBody *SecondBody, const FQuat& JointFrame1, AnimPhysTwistAxis TwistAxis, const FVector& InJointLimitMin, const FVector& InJointLimitMax, float InJointBias)
{
	FVector JointMinAngles = FMath::DegreesToRadians<FVector>(InJointLimitMin);
	FVector JointMaxAngles = FMath::DegreesToRadians<FVector>(InJointLimitMax);

	FQuat Rotation0to1 = JointFrame0.Inverse() * JointFrame1;

	FVector WorldTwistAxis;
	FVector RotationTwistAxis;

	// Get twist dependent info for calculating spin and twist
	switch (TwistAxis)
	{
	case AnimPhysTwistAxis::AxisX:
		WorldTwistAxis = FVector(1.0f, 0.0f, 0.0f);
		RotationTwistAxis = Rotation0to1.GetAxisX();
		break;
	case AnimPhysTwistAxis::AxisY:
		WorldTwistAxis = FVector(0.0f, 1.0f, 0.0f);
		RotationTwistAxis = Rotation0to1.GetAxisY();
		break;
	case AnimPhysTwistAxis::AxisZ:
		WorldTwistAxis = FVector(0.0f, 0.0f, 1.0f);
		RotationTwistAxis = Rotation0to1.GetAxisZ();
		break;
	default:
		checkf(false, TEXT("Invalid setting for constraint twist axis."));
		break;
	}

	// Calculate Spin and Twist
	FQuat Swing = FQuat::FindBetween(WorldTwistAxis, RotationTwistAxis);
	FQuat Twist = Swing.Inverse() * Rotation0to1;

	// Invert if necessary
	if (Swing.W < 0.0f)
	{
		Swing *= -1.0f;
	}

	if (Twist.W < 0.0f)
	{
		Twist *= -1.0f;
	}

	float MinAngle0 = 0.0f;
	float MaxAngle0 = 0.0f;
	float MinAngle1 = 0.0f;
	float MaxAngle1 = 0.0f;
	float Swing0 = 0.0f;
	float Swing1 = 0.0f;
	FVector FrameAxis0 = FVector(0.0f);
	FVector FrameAxis1 = FVector(0.0f);
	FVector FrameTwistAxis = FVector(0.0f);
	float TwistAmount = 0.0f;

	// Get twist dependent info to create the limits
	switch (TwistAxis)
	{
	case AnimPhysTwistAxis::AxisX:
		MinAngle0 = JointMinAngles.Y;
		MaxAngle0 = JointMaxAngles.Y;
		MinAngle1 = JointMinAngles.Z;
		MaxAngle1 = JointMaxAngles.Z;
		Swing0 = Swing.Y;
		Swing1 = Swing.Z;
		FrameAxis0 = JointFrame1.GetAxisY();
		FrameAxis1 = JointFrame1.GetAxisZ();
		FrameTwistAxis = JointFrame1.GetAxisX();
		TwistAmount = Twist.X;
		break;
	case AnimPhysTwistAxis::AxisY:
		MinAngle0 = JointMinAngles.X;
		MaxAngle0 = JointMaxAngles.X;
		MinAngle1 = JointMinAngles.Z;
		MaxAngle1 = JointMaxAngles.Z;
		Swing0 = Swing.X;
		Swing1 = Swing.Z;
		FrameAxis0 = JointFrame1.GetAxisX();
		FrameAxis1 = JointFrame1.GetAxisZ();
		FrameTwistAxis = JointFrame1.GetAxisY();
		TwistAmount = Twist.Y;
		break;
	case AnimPhysTwistAxis::AxisZ:
		MinAngle0 = JointMinAngles.X;
		MaxAngle0 = JointMaxAngles.X;
		MinAngle1 = JointMinAngles.Y;
		MaxAngle1 = JointMaxAngles.Y;
		Swing0 = Swing.X;
		Swing1 = Swing.Y;
		FrameAxis0 = JointFrame1.GetAxisX();
		FrameAxis1 = JointFrame1.GetAxisY();
		FrameTwistAxis = JointFrame1.GetAxisZ();
		TwistAmount = Twist.Z;
		break;
	default:
		checkf(false, TEXT("Invalid setting for constraint twist axis."));
		break;
	}

	if (MinAngle0 == MaxAngle0)
	{
		float TargetSwing = 2.0f * (-Swing0 + FMath::Sin(MinAngle0 / 2.0f)) / DeltaTime;
		LimitContainer.Add(FAnimPhysAngularLimit(FirstBody, SecondBody, FrameAxis0, TargetSwing));
	}
	else if (MaxAngle0 - MinAngle0 < FMath::DegreesToRadians(360.0f))
	{
		float TargetSwing1 = 2.0f * (-Swing0 + FMath::Sin(MinAngle0 / 2.0f)) / DeltaTime;
		float TargetSwing2 = 2.0f * (Swing0 - FMath::Sin(MaxAngle0 / 2.0f)) / DeltaTime;
		LimitContainer.Add(FAnimPhysAngularLimit(FirstBody, SecondBody, FrameAxis0, TargetSwing1, 0));
		LimitContainer.Add(FAnimPhysAngularLimit(FirstBody, SecondBody, -FrameAxis0, TargetSwing2, 0));
	}

	if (MinAngle1 == MaxAngle1)
	{
		float TargetSwing = InJointBias * 2.0f * (-Swing1 + MinAngle1) / DeltaTime;
		LimitContainer.Add(FAnimPhysAngularLimit(FirstBody, SecondBody, FrameAxis1, TargetSwing));
	}
	else if (MaxAngle1 - MinAngle1 < FMath::DegreesToRadians(360.0f))
	{
		float TargetSwing1 = 2.0f * (-Swing1 + FMath::Sin(MinAngle1 / 2.0f)) / DeltaTime;
		float TargetSwing2 = 2.0f * (Swing1 - FMath::Sin(MaxAngle1 / 2.0f)) / DeltaTime;
		LimitContainer.Add(FAnimPhysAngularLimit(FirstBody, SecondBody, FrameAxis1, TargetSwing1, 0));
		LimitContainer.Add(FAnimPhysAngularLimit(FirstBody, SecondBody, -FrameAxis1, TargetSwing2, 0));
	}

	float TargetTwist = InJointBias * 2.0f * -TwistAmount / DeltaTime;
	LimitContainer.Add(FAnimPhysAngularLimit(FirstBody, SecondBody, FrameTwistAxis, TargetTwist));
}

void FAnimPhys::ConstrainAngularRange(float DeltaTime, TArray<FAnimPhysAngularLimit>& LimitContainer, FAnimPhysRigidBody *FirstBody, FAnimPhysRigidBody *SecondBody, const FQuat& JointFrame, AnimPhysTwistAxis TwistAxis, const FVector& JointLimitMin, const FVector& JointLimitMax, float InJointBias)
{
	// a generic configurable 6dof style way to specify angular limits.  used for hard limits such as joint ranges.
	FQuat FirstBodyLocalFrame = (FirstBody) ? FirstBody->Pose.Orientation * JointFrame : JointFrame;
	FQuat SecondBodyOrientation = (SecondBody) ? SecondBody->Pose.Orientation : FQuat::Identity;

	return ConstrainAngularRangeInternal(DeltaTime, LimitContainer, FirstBody, FirstBodyLocalFrame, SecondBody, SecondBodyOrientation, TwistAxis, JointLimitMin, JointLimitMax, InJointBias);
}

void FAnimPhys::ConstrainConeAngle(float DeltaTime, TArray<FAnimPhysAngularLimit>& LimitContainer, FAnimPhysRigidBody* FirstBody, const FVector& Normal0, FAnimPhysRigidBody* SecondBody, const FVector& Normal1, float LimitAngle, float InJointBias) // a hinge is a cone with 0 limitangle
{
	check(SecondBody);

	bool ZeroLimit = LimitAngle == 0.0f;

	// First anchor in first body local space
	FVector WorldSpaceNormal0 = (FirstBody) ? FirstBody->Pose.Orientation.RotateVector(Normal0) : Normal0;
	// Second anchor in rb1 space, no check here as we asserted above
	FVector WorldSpaceNormal1 = SecondBody->Pose.Orientation.RotateVector(Normal1);
	FVector Axis = FVector::CrossProduct(WorldSpaceNormal1, WorldSpaceNormal0).GetSafeNormal();

	float BodyAngle = FMath::Acos(FMath::Clamp(FVector::DotProduct(WorldSpaceNormal0, WorldSpaceNormal1), 0.0f, 1.0f));
	float CurrentAngleDelta = BodyAngle - FMath::DegreesToRadians(LimitAngle);
	float TargetSpin = ZeroLimit ? InJointBias : 1.0f * CurrentAngleDelta / DeltaTime;

	LimitContainer.Add(FAnimPhysAngularLimit(FirstBody, SecondBody, Axis, TargetSpin, LimitAngle > 0.0f ? 0.0f : -MAX_flt, MAX_flt));
}

void FAnimPhys::ConstrainPlanar(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody* Body, const FTransform& PlaneTransform)
{
	FPlane LimitPlane(PlaneTransform.GetLocation(), PlaneTransform.GetUnitAxis(EAxis::Z));
	float DistanceFromPlane = LimitPlane.PlaneDot(Body->Pose.Position);

	// Reduce distance by radius if using a spherical collision method
	if(Body->CollisionType != AnimPhysCollisionType::CoM)
	{
		DistanceFromPlane -= Body->SphereCollisionRadius;
	}

	float TargetSpeed = DistanceFromPlane / DeltaTime;

	FVector Position0 = PlaneTransform.GetLocation();
	FVector Position1 = FVector::ZeroVector;

	LimitContainer.Add(FAnimPhysLinearLimit(nullptr, Body, Position0, Position1, LimitPlane.GetSafeNormal(), TargetSpeed, TargetSpeed, FVector2D(0.0f, MAX_flt)));
}

void FAnimPhys::ConstrainSphericalInner(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody* Body, const FTransform& SphereTransform, float SphereRadius)
{
	FVector SphereToBody = Body->Pose.Position - SphereTransform.GetLocation();
	FVector LimitNormal = SphereToBody.GetSafeNormal();
	float DistanceToLimit = SphereToBody.Size() - SphereRadius;

	if(Body->CollisionType != AnimPhysCollisionType::CoM)
	{
		DistanceToLimit += Body->SphereCollisionRadius;
	}

	float TargetSpeed = DistanceToLimit / DeltaTime;

	FVector Position0 = SphereTransform.GetLocation();
	FVector Position1 = FVector::ZeroVector;

	LimitContainer.Add(FAnimPhysLinearLimit(nullptr, Body, Position0, Position1, LimitNormal, TargetSpeed, TargetSpeed, FVector2D(-MAX_flt, 0.0f)));
}

void FAnimPhys::ConstrainSphericalOuter(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody* Body, const FTransform& SphereTransform, float SphereRadius)
{
	FVector SphereToBody = Body->Pose.Position - SphereTransform.GetLocation();
	float DistanceToLimit = SphereToBody.Size() - SphereRadius;

	if(Body->CollisionType != AnimPhysCollisionType::CoM)
	{
		DistanceToLimit -= Body->SphereCollisionRadius;
	}

	float TargetSpeed = DistanceToLimit / DeltaTime;

	FVector Position0 = SphereTransform.GetLocation();
	FVector Position1 = FVector::ZeroVector;

	LimitContainer.Add(FAnimPhysLinearLimit(nullptr, Body, Position0, Position1, SphereToBody.GetSafeNormal(), TargetSpeed, TargetSpeed, FVector2D(0.0f, MAX_flt)));
}

void FAnimPhys::CreateSpring(TArray<FAnimPhysSpring>& SpringContainer, FAnimPhysRigidBody* Body0, FVector Position0, FAnimPhysRigidBody* Body1, FVector Position1)
{
	FAnimPhysSpring NewSpring;
	NewSpring.Body0 = Body0;
	NewSpring.Body1 = Body1;
	NewSpring.Anchor0 = Position0;
	NewSpring.Anchor1 = Position1;
	NewSpring.SpringConstantLinear = AnimPhysicsConstants::DefaultSpringConstantLinear;
	NewSpring.SpringConstantAngular = AnimPhysicsConstants::DefaultSpringConstantAngular;

	SpringContainer.Add(NewSpring);
}

void FAnimPhys::InitializeBodyVelocity(float DeltaTime, FAnimPhysRigidBody *InBody, const FVector& GravityDirection)
{
	// Gather weak forces being applied to body at beginning of timestep
	// Forward euler update of the velocity and rotation/spin 
	InBody->PreviousState.Pose.Position = InBody->Pose.Position;
	InBody->PreviousState.Pose.Orientation = InBody->Pose.Orientation;

	float LinearDampingForBody = InBody->bLinearDampingOverriden ? InBody->LinearDamping : AnimPhysicsConstants::LinearDamping;
	float AngularDampingForBody = InBody->bAngularDampingOverriden ? InBody->AngularDamping : AnimPhysicsConstants::AngularDamping;
	float LinearDampingLeftOver = FMath::Pow(1.0f - LinearDampingForBody, DeltaTime);
	float AngularDampingLeftOver = FMath::Pow(1.0f - AngularDampingForBody, DeltaTime);

	InBody->LinearMomentum *= LinearDampingLeftOver;
	InBody->AngularMomentum *= AngularDampingLeftOver;

	FVector Force = InBody->bUseGravityOverride ? (InBody->GravityOverride * InBody->Mass) : (GravityDirection * FMath::Abs(UPhysicsSettings::Get()->DefaultGravityZ) * InBody->Mass * InBody->GravityScale);
	FVector Torque(0.0f, 0.0f, 0.0f);

	// Add wind forces
	if(InBody->bWindEnabled)
	{
		// Multiplier for wind forces, we have a similar arbitrary scale in cloth wind, doing similar here
		// so we get somewhat similar appearance between cloth and anim dynamics.
		// #TODO Maybe put this and the cloth scale into physics settings?
		static const float WindUnitScale = 250.0f;

		// Wind velocity in body space
		FVector WindVelocity = InBody->WindData.WindDirection * InBody->WindData.WindSpeed * WindUnitScale * InBody->WindData.BodyWindScale;

		if(WindVelocity.SizeSquared() > UE_SMALL_NUMBER)
		{
			Force += WindVelocity * InBody->WindData.WindAdaption;
		}
	}

	InBody->LinearMomentum += Force * DeltaTime;
	InBody->AngularMomentum += Torque * DeltaTime;

	FMatrix OrientationAsMatrix = FQuatRotationMatrix::Make(InBody->Pose.Orientation);

	InBody->InverseWorldSpaceTensor = OrientationAsMatrix * (InBody->InverseTensorWithoutMass * InBody->InverseMass) * OrientationAsMatrix.GetTransposed();
}

void FAnimPhys::CalculateNextPose(float DeltaTime, FAnimPhysRigidBody* InBody)
{
	const bool bUseRKOrientIntegration = true;

	// After an acceptable velocity and spin are computed, a forward euler update is applied to the position and orientation.
	InBody->NextPosition = InBody->Pose.Position + InBody->LinearMomentum * InBody->InverseMass * DeltaTime;

	if (bUseRKOrientIntegration)
	{
		InBody->NextOrientation = UpdateOrientRK(InBody->Pose.Orientation, InBody->InverseTensorWithoutMass * InBody->InverseMass, InBody->AngularMomentum, DeltaTime);
	}
	else
	{
		InBody->NextOrientation = (InBody->Pose.Orientation + DiffQ(InBody->Pose.Orientation, InBody->InverseTensorWithoutMass * InBody->InverseMass, InBody->AngularMomentum) * DeltaTime).GetNormalized();
	}
}

void FAnimPhys::UpdatePose(FAnimPhysRigidBody* InBody)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsPoseUpdate, bEnableDetailedStats);

	// after an acceptable velocity and spin are computed, a forward euler update is applied to the position and orientation.
	InBody->PreviousPosition = InBody->Pose.Position;  // should do this at beginning of physics loop in case someone teleports the body.
	InBody->PreviousOrientation = InBody->Pose.Orientation;
	InBody->Pose.Position = InBody->NextPosition;
	InBody->Pose.Orientation = InBody->NextOrientation;

	FMatrix OrientAsMatrix = FQuatRotationMatrix::Make(InBody->Pose.Orientation);
	InBody->InverseWorldSpaceTensor = OrientAsMatrix * InBody->InverseTensorWithoutMass * InBody->InverseMass * OrientAsMatrix.GetTransposed();
}

void FAnimPhys::PhysicsUpdate(float DeltaTime, TArray<FAnimPhysRigidBody*>& Bodies, TArray<FAnimPhysLinearLimit>& LinearLimits, TArray<FAnimPhysAngularLimit>& AngularLimits, TArray<FAnimPhysSpring>& Springs, const FVector& GravityDirection, const FVector& ExternalForce, const FVector& ExternalLinearAcc, const FVector& ExternalAngularAcc, const FVector& ExternalAngularVelocity, int32 NumPreIterations, int32 NumPostIterations)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsUpdate, bEnableDetailedStats);

	for (FAnimPhysRigidBody* Body : Bodies)
	{
		InitializeBodyVelocity(DeltaTime, Body, GravityDirection);	// based on previous and current force/torque
	}

	// Apply any external forces.
	if(!ExternalForce.IsNearlyZero())
	{
		for(FAnimPhysRigidBody* Body : Bodies)
		{
			Body->LinearMomentum += ExternalForce * DeltaTime;
		}
	}

	// Apply external acceleration
	if (!ExternalLinearAcc.IsNearlyZero())
	{
		for (FAnimPhysRigidBody* Body : Bodies)
		{
			if (Body->InverseMass > UE_KINDA_SMALL_NUMBER)
			{
				Body->LinearMomentum += ((ExternalLinearAcc / Body->InverseMass) * DeltaTime); // need to scale by mass to go from acc to momentum
			}
		}
	}

	// Apply fictitious Forces
	if(!(ExternalAngularVelocity.IsNearlyZero() && ExternalAngularAcc.IsNearlyZero()))
	{
		for (FAnimPhysRigidBody* Body : Bodies)
		{
			const FVector Velocity = Body->LinearMomentum * Body->InverseMass;
			const FVector AngularVelocity = Body->Spin();

			const FVector WxR = FVector::CrossProduct(ExternalAngularVelocity, Body->Pose.Position);
			const FVector WxWxR = FVector::CrossProduct(ExternalAngularVelocity, WxR);
			const FVector WxV = FVector::CrossProduct(ExternalAngularVelocity, Velocity);
			const FVector AxR = FVector::CrossProduct(ExternalAngularAcc, Body->Pose.Position);

			// Calculate the phantom forces
			const FVector CoriolisAcc = CoriolisAlpha * 2.0f * WxV;
			const FVector CentrifugalAcc = CentrifugalAlpha * WxWxR;
			const FVector EulerAcc = EulerAlpha * AxR;
			const FVector AngularAcc = AngularAccelerationAlpha * ExternalAngularAcc;

			// Apply the phantom forces
			Body->LinearMomentum -= (CoriolisAcc + CentrifugalAcc + EulerAcc) / Body->InverseMass * DeltaTime;
			Body->AngularMomentum -= Body->InertiaTensor.TransformPosition(AngularAcc) * DeltaTime;
		}
	}
	

	for (FAnimPhysSpring& Spring : Springs)
	{
		Spring.ApplyForces(DeltaTime);
	}
	
	for (int32 Iteration = 0; Iteration < NumPreIterations; ++Iteration)
	{
		{
			for(FAnimPhysLinearLimit& CurrentLimit : LinearLimits)
			{
				CurrentLimit.Iter(DeltaTime);
			}
		}

		{
			for (FAnimPhysAngularLimit& CurrentLimit : AngularLimits)
			{
				CurrentLimit.Iter(DeltaTime);
			}
		}
	}

	for (FAnimPhysRigidBody* Body : Bodies)
	{
		CalculateNextPose(DeltaTime, Body);
	}

	// The objective of this step is to take away any velocity that was added strictly for purposes of reaching the constraint or contact.
	// i.e. we added some velocity to a point to pull it away from something that was inter-penetrating but that velocity shouldn't stick around for the next frame,
	// otherwise we will have lots of jitter from our contacts and oscillations from constraints. Do this by clearing the targetvelocities and reinvoking the solver.
	for (FAnimPhysLinearLimit& CurrentLimit : LinearLimits)
	{
		CurrentLimit.RemoveBias();
	}

	for (FAnimPhysAngularLimit& CurrentLimit : AngularLimits)
	{
		CurrentLimit.RemoveBias();
	}

	for (int32 Iteration = 0; Iteration < NumPostIterations; ++Iteration)
	{
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsLinearPost, bEnableDetailedStats);
			for(FAnimPhysLinearLimit& CurrentLimit : LinearLimits)
			{
				CurrentLimit.Iter(DeltaTime);
			}
		}

		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsAngularPost, bEnableDetailedStats);
			for(FAnimPhysAngularLimit& CurrentLimit : AngularLimits)
			{
				CurrentLimit.Iter(DeltaTime);
			}
		}
	}

	for (FAnimPhysRigidBody* Body : Bodies)
	{
		UpdatePose(Body);
	}
}

void FAnimPhysSpring::ApplyForces(float DeltaTime)
{
	if (bApplyLinear)
	{
		// World space spring ends
		FVector Position0 = (Body0) ? Body0->GetPose() * Anchor0 : Anchor0;
		FVector Position1 = (Body1) ? Body1->GetPose() * Anchor1 : Anchor1;

		// World oriented impact points to apply impulses
		FVector OrientedPos0 = (Body0) ? Body0->Pose.Orientation * Anchor0 : Anchor0;
		FVector OrientedPos1 = (Body1) ? Body1->Pose.Orientation * Anchor1 : Anchor1;

		FVector P0ToP1 = Position1 - Position0;
		float SpringLength = P0ToP1.Size();
		float ScalarForce = -SpringConstantLinear * SpringLength * DeltaTime;
		FVector Impulse = P0ToP1.GetSafeNormal() * ScalarForce;

		// Apply linear spring forces
		if (Body0)
		{
			FAnimPhys::ApplyImpulse(Body0, OrientedPos0, -Impulse);
		}

		if (Body1)
		{
			FAnimPhys::ApplyImpulse(Body1, OrientedPos1, Impulse);
		}
	}

	if (bApplyAngular)
	{
		FVector Body1AngularAxis(0.0f);

		switch (AngularTargetAxis)
		{
		case AnimPhysTwistAxis::AxisX:
			Body1AngularAxis = Body1->GetPose().Orientation.GetAxisX();
			break;
		case AnimPhysTwistAxis::AxisY:
			Body1AngularAxis = Body1->GetPose().Orientation.GetAxisY();
			break;
		case AnimPhysTwistAxis::AxisZ:
			Body1AngularAxis = Body1->GetPose().Orientation.GetAxisZ(); //-V595
			break;
		default:
			checkf(false, TEXT("Invalid target axis option"));
			break;
		}

		FVector WorldSpaceTarget = TargetOrientationOffset.RotateVector(AngularTarget);
		FQuat ToTarget = FQuat::FindBetween(Body1AngularAxis, WorldSpaceTarget);

		float RotAngle(0.0f);
		FVector RotAxis(0.0f);
		ToTarget.ToAxisAndAngle(RotAxis, RotAngle);
		FVector AddedMomentum = RotAxis * (-SpringConstantAngular * RotAngle) * DeltaTime;

		if (Body0)
		{
			Body0->AngularMomentum += AddedMomentum;
		}

		if (Body1)
		{
			Body1->AngularMomentum -= AddedMomentum;
		}
	}
}

