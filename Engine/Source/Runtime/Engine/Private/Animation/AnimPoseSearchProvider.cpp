// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimPoseSearchProvider.h"
#include "Features/IModularFeatures.h"

namespace UE::Anim
{

FName IPoseSearchProvider::GetModularFeatureName()
{
	static FName FeatureName = FName("AnimPoseSearch");
	return FeatureName;
}
	
IPoseSearchProvider* IPoseSearchProvider::Get()
{
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;

	if (IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName()))
	{
		return &IModularFeatures::Get().GetModularFeature<IPoseSearchProvider>(GetModularFeatureName());
	}

	return nullptr;
}

bool IPoseSearchProvider::IsAvailable()
{
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	return IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName());
}

} // namespace UE::Anim