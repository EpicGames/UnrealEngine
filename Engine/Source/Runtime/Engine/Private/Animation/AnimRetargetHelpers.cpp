// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimRetargetHelpers.h"


#if WITH_EDITOR
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Animation/AnimSequence.h"
#include "Animation/PoseAsset.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Templates/UnrealTemplate.h"
#include "PackageTools.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/UObjectGlobals.h"
#include "CoreGlobals.h"
#include "Misc/FeedbackContext.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AnimRetargetHelpers"

namespace UE::Anim::RetargetHelpers
{

#if WITH_EDITOR

namespace Private
{

template<typename AssetType>
ERetargetSourceAssetStatus CheckRetargetSourceAssetDataImpl(const AssetType* InAsset)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TSoftObjectPtr<USkeletalMesh>& RetargetSourceAsset = InAsset->RetargetSourceAsset;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FAssetData AssetData;
	UE::AssetRegistry::EExists AssetDataStatus = RetargetSourceAsset.IsNull() 
		? UE::AssetRegistry::EExists::DoesNotExist
		: IAssetRegistry::GetChecked().TryGetAssetByObjectPath(RetargetSourceAsset.ToSoftObjectPath(), AssetData);

	if (InAsset->RetargetSourceAssetReferencePose.Num() > 0)
	{
		if (AssetDataStatus != AssetRegistry::EExists::Unknown)
		{
			if (AssetDataStatus == UE::AssetRegistry::EExists::Exists)
			{
				return ERetargetSourceAssetStatus::RetargetDataOk;
			}
			else // if (AssetDataStatus == UE::AssetRegistry::EExists::DoesNotExist)
			{
				UE_LOGF(LogAnimation, Warning, "Asset [%ls] references a missing Retarget Source Asset [%ls]. Retarget Reference Pose has [%d] elements. Please, add a correct retarget source asset and resave."
					, *InAsset->GetFullName()
					, *(RetargetSourceAsset.GetLongPackageName() + FString("/") + RetargetSourceAsset.GetAssetName())
					, InAsset->RetargetSourceAssetReferencePose.Num());
				return ERetargetSourceAssetStatus::RetargetSourceMissing;
			}
		}
		else // if Asset registry is not ready, we go with a slow path
		{
			const USkeletalMesh* SourceReferenceMesh = RetargetSourceAsset.LoadSynchronous();
			if (SourceReferenceMesh == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "Asset [%ls] references a missing Retarget Source Asset [%ls]. Retarget Reference Pose has [%d] elements. Please, add a correct retarget source asset and resave."
					, *InAsset->GetFullName()
					, *(RetargetSourceAsset.GetLongPackageName() + FString("/") + RetargetSourceAsset.GetAssetName())
					, InAsset->RetargetSourceAssetReferencePose.Num());
				return ERetargetSourceAssetStatus::RetargetSourceMissing;
			}
			else
			{
				return ERetargetSourceAssetStatus::RetargetDataOk;
			}
		}
	}

	return ERetargetSourceAssetStatus::NoRetargetDataSet;
}

void CheckRetargetSourceAssetData(bool bFixAssets, bool bWantsFullScan, const TArray<FString>& IncludedPaths, const TArray<FString>& ExcludedPaths)
{
	TArray<FAssetData> Assets;

	FARFilter AssetFilter;
	if (IncludedPaths.Num() > 0)
	{
		for (const FString& IncludedPath : IncludedPaths)
		{
			UE_LOGF(LogAnimation, Log, "Check Retarget Source Assets scan folder [%ls].", *IncludedPath);
			AssetFilter.PackagePaths.AddUnique(*IncludedPath);
		}
		AssetFilter.bRecursivePaths = true;
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "Check Retarget Source Assets will scan all folders (this might take some time and require a lot of memory).");
	}
	AssetFilter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
	AssetFilter.ClassPaths.Add(UPoseAsset::StaticClass()->GetClassPathName());

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.GetAssets(AssetFilter, Assets);

	UE_LOGF(LogAnimation, Log, "Check Retarget Source Assets found [%d] assets.", Assets.Num());

	// Run through paths and classes that should be excluded
	if (Assets.Num() > 0 && ExcludedPaths.Num() > 0)
	{
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.bRecursivePaths = true;
		for (const FString& ExcludedPath : ExcludedPaths)
		{
			UE_LOGF(LogAnimation, Log, "Check Retarget Source Assets Excluded folder : [%ls].", *ExcludedPath);
			Filter.PackagePaths.AddUnique(*ExcludedPath);
		}
		TArray<FAssetData> ExcludedAssetList;
		AssetRegistry.GetAssets(Filter, ExcludedAssetList);
		Assets.RemoveAll([&ExcludedAssetList](const FAssetData& Asset) {return ExcludedAssetList.Contains(Asset); });
	}

	UE_LOGF(LogAnimation, Log, "Check Retarget Source Assets after filtering exclusions : [%d] assets.", Assets.Num());

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FName SearchableName = GET_MEMBER_NAME_CHECKED(UAnimSequence, RetargetSourceAsset);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	const FString AssetNone("None");

	const int32 NumAssets = Assets.Num();
	TArray<UPackage*> PackagesToUnload;
	PackagesToUnload.Reserve(NumAssets);

	FScopedSlowTask Feedback(NumAssets, LOCTEXT("ScanningRetargetSourceAssets", "Scanning Retarget Source Assets..."));
	Feedback.MakeDialog(true);

	for (int32 Idx = 0; Idx < NumAssets; Idx++)
	{
		FAssetData& AssetData = Assets[Idx];

		Feedback.EnterProgressFrame(1, 
			FText::Format(LOCTEXT("ScanningRetargetSourceAssetsProgress", "Assets Left [{0}] - {1}"), 
				NumAssets - Idx,
				FText::FromString((AssetData.PackagePath.ToString() + FString("/") + AssetData.AssetName.ToString()))));

		// Fast scan will skip assets that does not have a retarget source asset set in the metadata
		// With full scan we can check if an anim sequence has retarget transforms but no asset set for some reason (but it is slower)
		if (!bWantsFullScan)
		{
			const FString RetargetSourceAssetFile = AssetData.GetTagValueRef<FString>(SearchableName);
			if (RetargetSourceAssetFile.IsEmpty() || RetargetSourceAssetFile == AssetNone)
			{
				continue;
			}
		}

		UPackage* Package = FindPackage(NULL, *AssetData.PackageName.ToString());
		bool bWasPackageFullyLoaded = Package && Package->IsFullyLoaded();

		if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(AssetData.GetAsset()))
		{
			Package = AnimSequence->GetPackage();
			const ERetargetSourceAssetStatus Status = Private::CheckRetargetSourceAssetDataImpl(AnimSequence);
			if (bFixAssets && Status == ERetargetSourceAssetStatus::RetargetSourceMissing)
			{
				AnimSequence->Modify();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				AnimSequence->RetargetSourceAsset.Reset();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				AnimSequence->RetargetSourceAssetReferencePose.Empty();
				AnimSequence->MarkPackageDirty();
				bWasPackageFullyLoaded = true; // avoid the package getting unloaded if it has been fixed
			}
		}
		else if (UPoseAsset* PoseAsset = Cast<UPoseAsset>(AssetData.GetAsset()))
		{
			Package = PoseAsset->GetPackage();
			const ERetargetSourceAssetStatus Status = Private::CheckRetargetSourceAssetDataImpl(PoseAsset);
			if (bFixAssets && Status == ERetargetSourceAssetStatus::RetargetSourceMissing)
			{
				PoseAsset->Modify();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				PoseAsset->RetargetSourceAsset.Reset();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				PoseAsset->RetargetSourceAssetReferencePose.Empty();
				PoseAsset->MarkPackageDirty();
				bWasPackageFullyLoaded = true; // avoid the package getting unloaded if it has been fixed
			}
		}

		if (!bWasPackageFullyLoaded && Package && Package->IsFullyLoaded())
		{
			PackagesToUnload.Add(Package);
		}

		const bool bCancelProcess = GWarn->ReceivedUserCancel();

		if (((Idx % 100) == 0 || (Idx == NumAssets - 1) || bCancelProcess) && PackagesToUnload.Num() > 0)
		{
			UPackageTools::UnloadPackages({ PackagesToUnload });
			PackagesToUnload.Reset();
		}

		if (bCancelProcess)
		{
			break;
		}
	}
}

static FAutoConsoleCommand CheckRetargetSourceAssetDataCmd(
	TEXT("a.CheckRetargetSourceAssetData"),
	TEXT("Checks if Anim Sequences and Pose Assets RetargetSourceAsset is valid. Type: 'a.CheckRetargetSourceAssetData /Game' to check assets in the Game (Content) folder.  'a.CheckRetargetSourceAssetData /Game true' to check and fix all the assets in the Game (Content) folder."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			static const FString IncludedPathsSwitch = "Include=";
			static const FString ExcludedPathsSwitch = "Exclude=";
			static const TCHAR* ParamDelims[] =
			{
				TEXT(";"),
				TEXT("+"),
				TEXT(","),
			};

			TArray<FString> IncludedPaths;
			TArray<FString> ExcludedPaths;
			const FString FixAssetsFlag("FixAssets");
			const FString FullScanFlag("FullScan");
			bool bWantsFix = false;
			bool bWantsFullScan = false;
			FString SwitchValue;

			const int32 NumArgs = Args.Num();
			for (int32 i = 0; i < NumArgs; i++)
			{
				const FString& Arg = Args[i];
				if (Arg.ToLower() == FixAssetsFlag)
				{
					bWantsFix = true;
				}
				else if (Arg.ToLower() == FullScanFlag)
				{
					bWantsFullScan = true;
				}
				else if (FParse::Value(*Arg, *IncludedPathsSwitch, SwitchValue))
				{
					SwitchValue.ParseIntoArray(IncludedPaths, ParamDelims, 3);
				}
				else if (FParse::Value(*Arg, *ExcludedPathsSwitch, SwitchValue))
				{
					SwitchValue.ParseIntoArray(ExcludedPaths, ParamDelims, 3);
				}
				else
				{
					IncludedPaths.AddUnique(Arg);
				}
			}

			CheckRetargetSourceAssetData(bWantsFix, bWantsFullScan, IncludedPaths, ExcludedPaths);
		}
	));
} // end namespace Private

ERetargetSourceAssetStatus CheckRetargetSourceAssetData(const UAnimSequence* InAsset)
{
	return Private::CheckRetargetSourceAssetDataImpl(InAsset);
}

ERetargetSourceAssetStatus CheckRetargetSourceAssetData(const UPoseAsset* InAsset)
{
	return Private::CheckRetargetSourceAssetDataImpl(InAsset);
}
#endif // WITH_EDITOR

} // end namespace UE::Anim::RetargetHelpers

#undef LOCTEXT_NAMESPACE 
