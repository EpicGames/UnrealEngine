// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimRootMotionProvider.h"
#include "Features/IModularFeatures.h"


namespace UE { namespace Anim {

const FName IAnimRootMotionProvider::ModularFeatureName(TEXT("AnimationWarping"));

const FName IAnimRootMotionProvider::AttributeName(TEXT("RootMotionDelta"));

const IAnimRootMotionProvider* IAnimRootMotionProvider::Get()
{
	if (IsAvailable())
	{
		IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
		return &IModularFeatures::Get().GetModularFeature<const IAnimRootMotionProvider>(ModularFeatureName);
	}
	return nullptr;
}

bool IAnimRootMotionProvider::IsAvailable()
{
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	return IModularFeatures::Get().IsModularFeatureAvailable(ModularFeatureName);
}

}}; // namespace UE::Anim