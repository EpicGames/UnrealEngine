// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimRuntimeTransformProviderData.h"
#include "SkinningSceneExtensionProxy.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "SceneInterface.h"
#include "UObject/AnimObjectVersion.h"

///////////////////////////////////////////////////////////////////////////////////////////////////

inline void TryAddTail(TIntrusiveDoubleLinkedList<FAnimRuntimeTrackTransformData>& List, FAnimRuntimeTrackTransformData* Data)
{
	if (Data)
	{
		List.AddTail(Data);
	}
}

FAnimRuntimeTransformProviderRenderData::~FAnimRuntimeTransformProviderRenderData()
{
	check(Proxies.Num() == 0);
	check(FreeList.IsEmpty());
}

void FAnimRuntimeTransformProviderRenderData::InsertProxy(FInstancedSkinningSceneExtensionProxy* Proxy)
{
	Proxies.FindOrAdd(Proxy);
}

void FAnimRuntimeTransformProviderRenderData::RemoveProxy(FInstancedSkinningSceneExtensionProxy* Proxy)
{
	Proxies.Remove(Proxy);
}

void FAnimRuntimeTransformProviderRenderData::Patch(FAnimRuntimeTrackPool::FPatch&& Patch)
{
	if (Patch.GetNumTracks() != Tracks.GetNumTracks())
	{
		for (FInstancedSkinningSceneExtensionProxy* Proxy : Proxies)
		{
			Proxy->SetUniqueAnimationCount(Patch.GetNumTracks());
		}
	}

	UE::TScopeLock Lock(FreeListMutex);

	Tracks.Patch<FAnimRuntimeTrackData>(
		MoveTemp(Patch),
		// Assignment
		[&] (FAnimRuntimeTrackData& Dst, const FAnimRuntimeTrackData& Src, bool bWasActive, int32 Index)
		{
			// Retire the previous transform data if it's not a duplicate of the current.
			if (Dst.Current != Dst.Previous)
			{
				TryAddTail(FreeList, Dst.Previous);
			}

			// Previous data is provided, free everything from the destination.
			if (Src.Previous)
			{
				TryAddTail(FreeList, Dst.Current);

				Dst.Previous = Src.Previous;
				Dst.Current  = Src.Current;
			}
			// Move current to previous on the destination and assign the new current.
			else
			{
				Dst.Previous = Dst.Current;
				Dst.Current  = Src.Current;
			}
		},
		// Deactivate
		[&] (FAnimRuntimeTrackData& Dst, int32 Index)
		{
			if (Dst.Previous != Dst.Current)
			{
				TryAddTail(FreeList, Dst.Previous);
			}
			TryAddTail(FreeList, Dst.Current);
			Dst = {};
		}
	);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FAnimRuntimeTransformProviderProxy::CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList)
{
	RenderData.InsertProxy(SceneExtensionProxy);

	AnimRuntimeTransformProvider = Scene.GetAnimRuntimeTransformProvider();

	if (AnimRuntimeTransformProvider)
	{
		AnimRuntimeTransformProvider->RegisterProxy(this);
	}
}

void FAnimRuntimeTransformProviderProxy::DestroyRenderThreadResources()
{
	RenderData.RemoveProxy(SceneExtensionProxy);

	if (AnimRuntimeTransformProvider)
	{
		AnimRuntimeTransformProvider->UnregisterProxy(this);
		AnimRuntimeTransformProvider = nullptr;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

bool UAnimRuntimeTransformProviderData::IsEnabled() const
{
	return Tracks.GetNumTracks() != 0 && Super::IsEnabled();
}

const FGuid& UAnimRuntimeTransformProviderData::GetTransformProviderID() const
{
	static FGuid GUID(ANIM_RUNTIME_TRANSFORM_PROVIDER_GUID);
	return GUID;
}

uint32 UAnimRuntimeTransformProviderData::GetUniqueAnimationCount() const
{
	return Tracks.GetNumTracks();
}

bool UAnimRuntimeTransformProviderData::UsesSkeletonBatching() const
{
	return false;
}

bool UAnimRuntimeTransformProviderData::HasAnimationBounds() const
{
	return false;
}

bool UAnimRuntimeTransformProviderData::GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const
{
	return false;
}

uint32 UAnimRuntimeTransformProviderData::GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const
{
	const uint32 AnimationIndex = InstanceData.AnimationIndex;
	if (AnimationIndex < uint32(Tracks.GetNumTracks()))
	{
		return AnimationIndex * 2u;
	}

	return 0u;
}

void UAnimRuntimeTransformProviderData::BeginDestroy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimRuntimeTransformProviderData::BeginDestroy);
	check(RenderData);

	// Push any pending changes so that the render thread can clean up everything.
	if (Tracks.GetNumDirtyTracks() > 0)
	{
		SubmitChanges();
	}

	ENQUEUE_RENDER_COMMAND(AnimRuntimeTransformProviderData_Release)([RenderData = MoveTemp(RenderData), FreeList = MoveTemp(FreeList), NumPoolInstances = NumPoolInstances](FRHICommandList&) mutable
	{
		FreeList.AddTail(MoveTemp(RenderData->FreeList));
		RenderData->FreeList.Reset();

		// Gather all active data and free it.
		RenderData->Tracks.EnumerateActiveDirtyTracks([&] (int32 Index)
		{
			FAnimRuntimeTrackData& Data = RenderData->Tracks.GetData(Index);
			if (Data.Current != Data.Previous)
			{
				TryAddTail(FreeList, Data.Previous);
			}
			TryAddTail(FreeList, Data.Current);
			Data = {};
		});

		int32 NumFreed = 0;
		while (FAnimRuntimeTrackTransformData* Data = FreeList.PopTail())
		{
			delete Data;
			NumFreed++;
		}
		check(NumFreed == NumPoolInstances);
	});

	FreeList.Reset();
	Tracks.Empty();
	Super::BeginDestroy();
}

void UAnimRuntimeTransformProviderData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Super::Serialize(Ar);
	Tracks.Serialize(Ar);
	Ar << NumTransforms;

	if (Ar.IsLoading())
	{
		RenderData->Tracks = Tracks;
	}
}

FTransformProviderRenderProxy* UAnimRuntimeTransformProviderData::CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const
{
	check(RenderData);
	return new FAnimRuntimeTransformProviderProxy(ExtensionProxy, *RenderData);
}

UAnimRuntimeTransformProviderData* UAnimRuntimeTransformProviderData::CreateAnimRuntimeTransformProviderData(UInstancedSkinnedMeshComponent* Owner)
{
	if (IsValid(Owner))
	{
		if (USkinnedAsset* SkinnedAsset = Owner->GetSkinnedAsset())
		{
			UAnimRuntimeTransformProviderData* Instance = NewObject<UAnimRuntimeTransformProviderData>(Owner);
			Instance->NumTransforms = SkinnedAsset->GetRefBasesInvMatrix().Num();
			return Instance;
		}
	}

	return nullptr;
}

void UAnimRuntimeTransformProviderData::SubmitChanges()
{
	check(RenderData);

	// Attempt to reclaim the free list from the last patch operation on the render thread. Include the render command as well
	// since SubmitChanges is called during the ISKM instance data update. Guard against multiple concurrent updates.
	UE::TScopeLock Lock(RenderData->FreeListMutex);
	FreeList.AddTail(MoveTemp(RenderData->FreeList));

	if (Tracks.GetNumDirtyTracks() > 0)
	{
		ENQUEUE_RENDER_COMMAND(AnimRuntimeTransformProviderDataSubmit)([&RenderData = *RenderData, Patch = Tracks.Finalize()](FRHICommandList&) mutable
		{
			RenderData.Patch(MoveTemp(Patch));
		});
	}
}

FAnimRuntimeTrackData UAnimRuntimeTransformProviderData::AcquireTrackData(EPreviousBoneTransformUpdateMode UpdateMode)
{
	const auto AcquireData = [&]
	{
		if (!FreeList.IsEmpty())
		{
			return FreeList.PopTail();
		}
		else
		{
			NumPoolInstances++;
			return new FAnimRuntimeTrackTransformData(NumTransforms);
		}
	};

	FAnimRuntimeTrackData TrackData;
	TrackData.Current = AcquireData();
	if (UpdateMode == EPreviousBoneTransformUpdateMode::UpdatePrevious)
	{
		TrackData.Previous = AcquireData();
	}
	else if (UpdateMode == EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious)
	{
		TrackData.Previous = TrackData.Current;
	}
	return TrackData;
}

int32 UAnimRuntimeTransformProviderData::AllocateTrack()
{
	return Tracks.AllocateTrack(FAnimRuntimeTrackData{});
}

FAnimRuntimeTrackAllocation UAnimRuntimeTransformProviderData::AllocateTrack(EPreviousBoneTransformUpdateMode UpdateMode)
{
	FAnimRuntimeTrackData TrackData = AcquireTrackData(UpdateMode);

	FAnimRuntimeTrackAllocation Allocation;
	Allocation.Index    = Tracks.AllocateTrack(TrackData);
	Allocation.Current  = TrackData.Current;
	if (UpdateMode != EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious)
	{
		Allocation.Previous = TrackData.Previous;
	}
	return Allocation;
}

FAnimRuntimeTrackAllocation UAnimRuntimeTransformProviderData::UpdateTrack(int32 TrackIndex, EPreviousBoneTransformUpdateMode UpdateMode)
{
	FAnimRuntimeTrackAllocation Allocation;

	if (!Tracks.IsActiveIndex(TrackIndex))
	{
		return Allocation;
	}

	FAnimRuntimeTrackData& TrackData = Tracks.GetData(TrackIndex);

	// Track was already dirty. We are updating multiple times without submitting changes.
	if (Tracks.IsDirtyIndex(TrackIndex))
	{
		if (TrackData.Previous != TrackData.Current)
		{
			TryAddTail(FreeList, TrackData.Previous);
		}
		TryAddTail(FreeList, TrackData.Current);
	}

	TrackData = AcquireTrackData(UpdateMode);
	const bool bSuccess = Tracks.UpdateTrack(TrackIndex, TrackData);
	check(bSuccess);

	Allocation.Index    = TrackIndex;
	Allocation.Current  = TrackData.Current;
	if (UpdateMode != EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious)
	{
		Allocation.Previous = TrackData.Previous;
	}
	return Allocation;
}

bool UAnimRuntimeTransformProviderData::DeallocateTrack(int32 TrackIndex)
{
	return Tracks.DeallocateTrack(TrackIndex);
}

///////////////////////////////////////////////////////////////////////////////