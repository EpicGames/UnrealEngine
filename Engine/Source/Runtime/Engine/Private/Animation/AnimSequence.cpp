// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimSequence.cpp: Skeletal mesh animation functions.
=============================================================================*/ 

#include "Animation/AnimSequence.h"

#include "Algo/Find.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "UObject/ObjectSaveContext.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "UObject/Package.h"
#include "Animation/AnimStats.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Animation/MirrorDataTable.h"
#include "UObject/UObjectIterator.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "EngineUtils.h"
#include "AnimationUtils.h"
#include "BonePose.h"
#include "Animation/AnimCompress.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimCurveCompressionCodec.h"
#include "Animation/AnimCurveCompressionSettings.h"
#include "Animation/VariableFrameStrippingSettings.h"
#include "EditorFramework/AssetImportData.h"
#include "Logging/MessageLog.h"
#include "DerivedDataCacheInterface.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectThreadContext.h"
#include "HAL/PlatformApplicationMisc.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "Animation/AnimStreamable.h"
#include "ProfilingDebugging/CookStats.h"
#include "Animation/AnimationPoseData.h"
#include "ITimeManagementModule.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/Skeleton.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

LLM_DEFINE_TAG(SequenceData);


#if WITH_EDITOR
#include "Animation/AnimationSequenceCompiler.h"
#include "Animation/AnimationCompressionDerivedData.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Cooker/CookDeterminismHelper.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/MemoryHasher.h"
#include "Misc/DataValidation.h"
#endif // WITH_EDITOR

#include "Animation/AnimSequenceHelpers.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequence)

#define USE_SLERP 0
#define LOCTEXT_NAMESPACE "AnimSequence"

DECLARE_CYCLE_STAT(TEXT("AnimSeq GetBonePose"), STAT_AnimSeq_GetBonePose, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("AnimSeq EvalCurveData"), STAT_AnimSeq_EvalCurveData, STATGROUP_Anim);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

int32 GPerformFrameStripping = 0;
int32 GPerformFrameStrippingOddFramedAnimations = 0;

static const TCHAR* StripFrameCVarName = TEXT("a.StripFramesOnCompression");
static const TCHAR* OddFrameStripStrippingCVarName = TEXT("a.StripOddFramesWhenFrameStripping");

static FAutoConsoleVariableRef CVarFrameStripping(
	StripFrameCVarName,
	GPerformFrameStripping,
	TEXT("1 = Strip every other frame on animations that have an even number of frames. 0 = off"));

static FAutoConsoleVariableRef CVarOddFrameStripping(
	OddFrameStripStrippingCVarName,
	GPerformFrameStrippingOddFramedAnimations,
	TEXT("1 = When frame stripping apply to animations with an odd number of frames too. 0 = only even framed animations"));

int32 GStripAdditiveRefPose = 0;
static FAutoConsoleVariableRef CVarStripAdditiveRefPose(
	TEXT("a.StripAdditiveRefPose"),
	GStripAdditiveRefPose,
	TEXT("1 = Strip additive ref poses on cook. 0 = off"));

namespace UE::Anim::Private
{
	bool ShouldStripAdditiveRefPose()
	{
		return GStripAdditiveRefPose != 0;
	}
} // namespace UE::Anim::Private

#if WITH_EDITOR

template <typename ArrayType>
FGuid GetArrayGuid(TArrayView<const ArrayType> Array)
{
	FSHA1 Sha;
	Sha.Update((uint8*)Array.GetData(), Array.Num() * Array.GetTypeSize());

	Sha.Final();

	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	FGuid Guid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	return Guid;
}

void OnCVarsChanged()
{
	if (GIsInitialLoad)
	{
		return; // not initialized
	}

	/*static TArray<UAnimSequence*> SequenceCache;
	static FString OutputMessage;
	
	SequenceCache.Reset();

	for (TObjectIterator<UAnimSequence> It; It; ++It)
	{
		SequenceCache.Add(*It);
	}

	SequenceCache.Sort([](const UAnimSequence& A, const UAnimSequence& B)
	{
		return A.GetFName() > B.GetFName();
	});

	OutputMessage.Reset();

	for (UAnimSequence* Seq : SequenceCache)
	{
		const FCompressedAnimSequence& AnimData = Seq->CompressedData;
		const FUECompressedAnimData& UEAnimData = AnimData.CompressedDataStructure;
		const int32 Additive = Seq->IsValidAdditive() ? 1 : 0;
		OutputMessage += FString::Printf(TEXT("%s - %.2f Fr:%i Add:%i TO:%i SO:%i CBS:%i\n"), *Seq->GetName(), Seq->SequenceLength, Seq->GetRawNumberOfFrames(), Additive, UEAnimData.CompressedTrackOffsets.Num(), UEAnimData.CompressedScaleOffsets.OffsetData.Num(), UEAnimData.CompressedByteStream.Num());
		OutputMessage += FString::Printf(TEXT("\t K:%i (%i : %i : %i)\n"), (int32)UEAnimData.KeyEncodingFormat, (int32)UEAnimData.TranslationCompressionFormat, (int32)UEAnimData.RotationCompressionFormat, (int32)UEAnimData.ScaleCompressionFormat);
		OutputMessage += FString::Printf(TEXT("\t Curve Codec:%s\n"), AnimData.CurveCompressionCodec ? *AnimData.CurveCompressionCodec->GetPathName() : TEXT("nullptr"));
		OutputMessage += FString::Printf(TEXT("\t TrackOff:%s\n"), *GetArrayGuid<int32>(UEAnimData.CompressedTrackOffsets).ToString());
		OutputMessage += FString::Printf(TEXT("\t ScaleOff:%s\n"), *GetArrayGuid<int32>(UEAnimData.CompressedScaleOffsets.OffsetData).ToString());
		OutputMessage += FString::Printf(TEXT("\t BoneByteStream:%s\n"), *GetArrayGuid<uint8>(UEAnimData.CompressedByteStream).ToString());
		OutputMessage += FString::Printf(TEXT("\t CurveByteStream:%s\n"), *GetArrayGuid<uint8>(AnimData.CompressedCurveByteStream).ToString());
	}

	OutputMessage += FString::Printf(TEXT("\n\nTotalAnims: %i"), SequenceCache.Num());
	FPlatformApplicationMisc::ClipboardCopy(*OutputMessage);*/

	static bool bFirstRun = true;

	static bool bCompressionFrameStrip = (GPerformFrameStripping == 1);
	static bool bOddFramedStrip = (GPerformFrameStrippingOddFramedAnimations == 1);

	static TArray<UAnimSequence*> SequenceCache;
	static FString OutputMessage;

	const bool bCurrentFrameStrip = (GPerformFrameStripping == 1);
	const bool bCurrentOddFramedStrip = (GPerformFrameStrippingOddFramedAnimations == 1);

	const bool bFrameStripChanged = bCompressionFrameStrip != bCurrentFrameStrip;
	const bool bOddFrameStripChanged = bOddFramedStrip != bCurrentOddFramedStrip;

	if (bFrameStripChanged || bOddFrameStripChanged)
	{
		bCompressionFrameStrip = bCurrentFrameStrip;
		bOddFramedStrip = bCurrentOddFramedStrip;

		SequenceCache.Reset();

		if (!bFirstRun) // No need to do this on the first run, only subsequent runs as temp anim sequences from compression may still be around
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
		bFirstRun = false;

		for (TObjectIterator<UAnimSequence> It; It; ++It)
		{
			SequenceCache.Add(*It);
		}

		if (SequenceCache.Num() == 0)
		{
			return; // Nothing to do
		}

		TArray< TPair<int32, UAnimSequence*> > Sizes;
		
		// Rebake/compress the animations
		for (UAnimSequence* Seq : SequenceCache)
		{
			Seq->CacheDerivedDataForCurrentPlatform();
			Sizes.Emplace(Seq->GetApproxCompressedSize(), Seq);
		}

		Sizes.Sort([](const TPair<int32, UAnimSequence*>& A, const TPair<int32, UAnimSequence*>& B)
		{
			return A.Key > B.Key;
		});

		OutputMessage.Reset();

		const TCHAR* StripMessage = bCompressionFrameStrip ? TEXT("Stripping: On") : TEXT("Stripping: Off");
		const TCHAR* OddMessage = bOddFramedStrip ? TEXT("Odd Frames: On") : TEXT("Odd Frames: Off");

		OutputMessage += FString::Printf(TEXT("%s - %s\n\n"), StripMessage, OddMessage);

		int32 TotalSize = 0;
		int32 NumAnimations = 0;
		for (const TPair<int32, UAnimSequence*>& Pair : Sizes)
		{
			const bool bIsOddFramed = (Pair.Value->GetNumberOfSampledKeys() % 2) == 0;
			if (bIsOddFramed)
			{
				OutputMessage += FString::Printf(TEXT("%s - %.1fK\n"), *Pair.Value->GetPathName(), (float)Pair.Key / 1000.f);
				TotalSize += Pair.Key;
				NumAnimations++;
			}
		}

		OutputMessage += FString::Printf(TEXT("\n\nTotalAnims: %i TotalSize = %.1fK"), NumAnimations, ((float)TotalSize / 1000.f));
		FPlatformApplicationMisc::ClipboardCopy(*OutputMessage);
	}
}

FAutoConsoleVariableSink AnimationCVarSink(FConsoleCommandDelegate::CreateStatic(&OnCVarsChanged));

FString GetAnimSequenceSpecificCacheKeySuffix(const UAnimSequence& Seq, bool bPerformStripping, float CompressionErrorThresholdScale, const ITargetPlatform* TargetPlatform)
{
	//Make up our content key consisting of:
	//	* Global animation compression version
	//  * Whether to strip frames
	//	* Our raw data GUID
	//	* Our skeleton GUID: If our skeleton changes our compressed data may now be stale
	//	* Baked Additive Flag
	//	* Additive ref pose GUID or hardcoded string if not available
	//	* Compression Settings
	//	* Curve compression settings
	//  * Target frames per second

	bool bIsValidAdditive = Seq.IsValidAdditive();
	char AdditiveType = bIsValidAdditive ? NibbleToTChar(Seq.AdditiveAnimType) : '0';
	char RefType = bIsValidAdditive ? NibbleToTChar(Seq.RefPoseType) : '0';

	FArcToHexString ArcToHexString;

	ArcToHexString.Ar << CompressionErrorThresholdScale;
	ArcToHexString.Ar << bPerformStripping;
	Seq.BoneCompressionSettings->PopulateDDCKey(UE::Anim::Compression::FAnimDDCKeyArgs(Seq, TargetPlatform), ArcToHexString.Ar);
	Seq.CurveCompressionSettings->PopulateDDCKey(ArcToHexString.Ar);
	Seq.VariableFrameStrippingSettings->PopulateDDCKey(UE::Anim::Compression::FAnimDDCKeyArgs(Seq, TargetPlatform), ArcToHexString.Ar);

	FString Ret = FString::Printf(TEXT("%i_%s%s%s_%c%c%i_%s_%s_%i"),
		Seq.CompressCommandletVersion,
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		*Seq.GetDataModel()->GenerateGuid().ToString(),
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		*Seq.GetSkeleton()->GetGuid().ToString(),
		*Seq.GetSkeleton()->GetVirtualBoneGuid().ToString(),
		AdditiveType,
		RefType,
		Seq.RefFrameIndex,
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		(bIsValidAdditive && Seq.RefPoseSeq) ? *Seq.RefPoseSeq->GetDataModel()->GenerateGuid().ToString() : TEXT("NoAdditiveGuid"),
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		*ArcToHexString.MakeString(),
		Seq.GetSamplingFrameRate().Numerator / Seq.GetSamplingFrameRate().Denominator
	);

	return Ret;
}
#endif

FName MarkerOrMirroredName(FName MarkerToFind, const UMirrorDataTable* MirrorTable)
{
	if (MirrorTable)
	{
		const FName* MirroredName = MirrorTable->SyncToMirrorSyncMap.Find(MarkerToFind);
		if (MirroredName)
		{
			return *MirroredName;
		}
	}
	return MarkerToFind;
}

/////////////////////////////////////////////////////
// FRawAnimSequenceTrackNativeDeprecated

//@deprecated with VER_REPLACED_LAZY_ARRAY_WITH_UNTYPED_BULK_DATA
struct FRawAnimSequenceTrackNativeDeprecated
{
	TArray<FVector> PosKeys;
	TArray<FQuat> RotKeys;
	friend FArchive& operator<<(FArchive& Ar, FRawAnimSequenceTrackNativeDeprecated& T)
	{
		return	Ar << T.PosKeys << T.RotKeys;
	}
};

/////////////////////////////////////////////////////
// FCurveTrack

/** Returns true if valid curve weight exists in the array*/
bool FCurveTrack::IsValidCurveTrack()
{
	bool bValid = false;

	if ( CurveName != NAME_None )
	{
		for (int32 I=0; I<CurveWeights.Num(); ++I)
		{
			// it has valid weight
			if (CurveWeights[I]>UE_KINDA_SMALL_NUMBER)
			{
				bValid = true;
				break;
			}
		}
	}

	return bValid;
}

/** This is very simple cut to 1 key method if all is same since I see so many redundant same value in every frame 
 *  Eventually this can get more complicated 
 *  Will return true if compressed to 1. Return false otherwise
 **/
bool FCurveTrack::CompressCurveWeights()
{
	// if always 1, no reason to do this
	if (CurveWeights.Num() > 1)
	{
		bool bCompress = true;
		// first weight
		float FirstWeight = CurveWeights[0];

		for (int32 I=1; I<CurveWeights.Num(); ++I)
		{
			// see if my key is same as previous
			if (fabs(FirstWeight - CurveWeights[I]) > UE_SMALL_NUMBER)
			{
				// if not same, just get out, you don't like to compress this to 1 key
				bCompress = false;
				break;
			}
		} 

		if (bCompress)
		{
			CurveWeights.Empty();
			CurveWeights.Add(FirstWeight);
			CurveWeights.Shrink();
		}

		return bCompress;
	}

	// nothing changed
	return false;
}

/////////////////////////////////////////////////////
// UAnimSequence

UAnimSequence::UAnimSequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Interpolation(EAnimInterpolationType::Linear)
	, bEnableRootMotion(false)
	, RootMotionRootLock(ERootMotionRootLock::RefPose)
	, bUseNormalizedRootMotionScale(true)
	, bRootMotionSettingsCopiedFromMontage(false)
#if WITH_EDITOR
	, bBlockCompressionRequests(false)
#endif
{
	RateScale = 1.0;

#if WITH_EDITORONLY_DATA
	ImportFileFramerate = 0.0f;
	ImportResampleFramerate = 0;
	bAllowFrameStripping = true;
	CompressionErrorThresholdScale = 1.f;

	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject| EObjectFlags::RF_NeedLoad))
	{
		check(DataModelInterface.GetObject());
	}
#endif
}

void UAnimSequence::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
	MarkerDataUpdateCounter = 0;
#endif
	Super::PostInitProperties();
}

static const FName AssetRegistryTag_CompressionRatio(TEXT("Compression Ratio"));
static const FName AssetRegistryTag_SourceFrameRate(TEXT("Source Frame Rate"));
static const FName AssetRegistryTag_NumberOfFrames(TEXT("Number of Frames"));
static const FName AssetRegistryTag_NumberOfKeys(TEXT("Number of Keys"));
static const FName AssetRegistryTag_CompressedSize(TEXT("Compressed Size (KB)"));
static const FName AssetRegistryTag_TargetFrameRate(TEXT("Target Frame Rate"));

void UAnimSequence::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}

	if (DataModelInterface.GetObject() && DataModelInterface->HasBeenPopulated())
	{
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_CompressionRatio, FString::Printf(TEXT("%.03f"), (double)GetApproxCompressedSize() / (double)GetUncompressedRawSize()), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_SourceFrameRate, FString::Printf(TEXT("%.2f"), DataModelInterface->GetFrameRate().AsDecimal()), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_NumberOfFrames, FString::Printf(TEXT("%.i"), DataModelInterface->GetNumberOfFrames()), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_NumberOfKeys, FString::Printf(TEXT("%i"), DataModelInterface->GetNumberOfKeys()), FAssetRegistryTag::TT_Numerical));
	}
	else
	{
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_SourceFrameRate, TEXT("0.0"), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_NumberOfFrames, TEXT("0"), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_NumberOfKeys, TEXT("0"), FAssetRegistryTag::TT_Numerical));
	}

	// Output unique sync marker names we use
	TStringBuilder<256> SyncMarkersBuilder;
	SyncMarkersBuilder.Append(USkeleton::AnimSyncMarkerTagDelimiter);

	for(FName SyncMarker : UniqueMarkerNames)
	{
		SyncMarkersBuilder.Append(SyncMarker.ToString());
		SyncMarkersBuilder.Append(USkeleton::AnimSyncMarkerTagDelimiter);
	}

	Context.AddTag(FAssetRegistryTag(USkeleton::AnimSyncMarkerTag, SyncMarkersBuilder.ToString(), FAssetRegistryTag::TT_Hidden));
#endif

	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_CompressedSize, FString::Printf(TEXT("%.02f"), (float)GetApproxCompressedSize() / 1024.0f), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_TargetFrameRate, FString::Printf(TEXT("%.2f"), PlatformTargetFrameRate.Default.AsDecimal()), FAssetRegistryTag::TT_Numerical));
	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR

void UAnimSequence::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	OutMetadata.Add(AssetRegistryTag_CompressionRatio, FAssetRegistryTagMetadata().SetDisplayName(LOCTEXT("CompressionRatio", "Compression Ratio")));
	OutMetadata.Add(AssetRegistryTag_SourceFrameRate, FAssetRegistryTagMetadata().SetDisplayName(LOCTEXT("SourceFrameRate", "Source Frame Rate")));
	OutMetadata.Add(AssetRegistryTag_NumberOfFrames, FAssetRegistryTagMetadata().SetDisplayName(LOCTEXT("NumberOfFrames", "Number of Frames")));
	OutMetadata.Add(AssetRegistryTag_NumberOfKeys, FAssetRegistryTagMetadata().SetDisplayName(LOCTEXT("NumberOfKeys", "Number of Keys")));
	OutMetadata.Add(AssetRegistryTag_CompressedSize, FAssetRegistryTagMetadata().SetDisplayName(LOCTEXT("CompressedSize", "Compressed Size (KB)")));
	OutMetadata.Add(AssetRegistryTag_TargetFrameRate, FAssetRegistryTagMetadata().SetDisplayName(LOCTEXT("TargetFrameRate", "Target Frame Rate")));
}

#endif // WITH_EDITOR

void UAnimSequence::AddReferencedObjects(UObject* This, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(This, Collector);

	UAnimSequence* AnimSeq = CastChecked<UAnimSequence>(This);
	UE_COMPRESSED_DATA_READ_SCOPE(AnimSeq);
	FCompressedAnimSequence& ThisCompressedData = AnimSeq->GetCompressedData_Internal();
	Collector.AddReferencedObject(ThisCompressedData.BoneCompressionCodec);
	Collector.AddReferencedObject(ThisCompressedData.CurveCompressionCodec);
}

#if WITH_EDITOR

int32 GAllowClearingCompressedDataDuringCook = 1;
static FAutoConsoleVariableRef CVarAllowClearingCompressedDataDuringCook(
	TEXT("a.AllowClearingCompressedDataDuringCook"),
	GAllowClearingCompressedDataDuringCook,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether or not to compressed data is cleared/free-ed during cook."),
	ECVF_Default);

void UAnimSequence::WillNeverCacheCookedPlatformDataAgain()
{
	UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation({this});
	bShouldClearCompressedData = GAllowClearingCompressedDataDuringCook == 1;
	if (bShouldClearCompressedData || !IsRunningCookCommandlet())
	{
		ClearAllCompressionData();
	}
}

void UAnimSequence::ClearAllCachedCookedPlatformData()
{
	if (GAllowClearingCompressedDataDuringCook || !IsRunningCookCommandlet())
	{
		ClearAllCompressionData();
	}
}

int64 UAnimSequence::GetUncompressedRawSize() const
{
	if (DataModelInterface.GetObject())
	{
		const int64 BoneRawSize = FRawAnimSequenceTrack::SingleKeySize * int64(DataModelInterface->GetNumBoneTracks()) * int64(DataModelInterface->GetNumberOfKeys());
		int64 CurveRawSize = 0;

		for (const FFloatCurve& Curve : DataModelInterface->GetCurveData().FloatCurves)
		{
			CurveRawSize += sizeof(FFloatCurve);
			CurveRawSize += sizeof(FRichCurveKey) * int64(Curve.FloatCurve.Keys.Num());
		}
		return BoneRawSize + CurveRawSize;
	}
	return 0;
}

int64 UAnimSequence::GetApproxRawSize() const
{
	return GetApproxBoneRawSize() + GetApproxCurveRawSize();
}

int64 UAnimSequence::GetApproxBoneRawSize() const
{
	int64 Total = 0;

	if (ShouldDataModelBeValid())
	{
		ValidateModel();

		const int64 NumberOfTracks = DataModelInterface->GetNumBoneTracks();

		Total = sizeof(FRawAnimSequenceTrack) * NumberOfTracks;
		Total += int64(DataModelInterface->GetNumberOfKeys()) * sizeof(FVector3f) * NumberOfTracks;
		Total += int64(DataModelInterface->GetNumberOfKeys()) * sizeof(FQuat4f) * NumberOfTracks;
		Total += int64(DataModelInterface->GetNumberOfKeys()) * sizeof(FVector3f) * NumberOfTracks;
	}

	return Total;
}

int64 UAnimSequence::GetApproxCurveRawSize() const
{
	int64 Total = 0;

	if (ShouldDataModelBeValid())
	{
		ValidateModel();

		for (const FFloatCurve& Curve : DataModelInterface->GetCurveData().FloatCurves)
		{
			Total += sizeof(FFloatCurve);
			Total += sizeof(FRichCurveKey) * int64(Curve.FloatCurve.Keys.Num());
		}
	}

	return Total;
}
#endif // WITH_EDITOR

int32 UAnimSequence::GetApproxBoneCompressedSize() const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	return GetApproxBoneCompressedSize_Lockless();
}

int32 UAnimSequence::GetApproxCompressedSize() const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	return GetApproxCompressedSize_Lockless();
}

int32 UAnimSequence::GetApproxBoneCompressedSize_Lockless() const
{
	return GetCompressedData_Internal().CompressedDataStructure != nullptr ? GetCompressedData_Internal().CompressedDataStructure->GetApproxCompressedSize(): 0;
}

int32 UAnimSequence::GetApproxCompressedSize_Lockless() const
{
	const int32 BoneTotal = GetApproxBoneCompressedSize_Lockless();
	int32 CurveTotal = GetCompressedData_Internal().CompressedCurveByteStream.Num();
	return BoneTotal + CurveTotal;
}

UAnimSequence::FScopedCompressedAnimSequence UAnimSequence::GetCompressedData() const
{
	const FCompressedAnimSequence& PlatformCompressedData = GetCompressedData_Internal();
	return FScopedCompressedAnimSequence(this, &PlatformCompressedData);
}

/**
 * Deserializes old compressed track formats from the specified archive.
 */
static void LoadOldCompressedTrack(FArchive& Ar, FCompressedTrack& Dst, int32 ByteStreamStride)
{
	// Serialize from the archive to a buffer.
	int32 NumBytes = 0;
	Ar << NumBytes;

	TArray<uint8> SerializedData;
	SerializedData.Empty( NumBytes );
	SerializedData.AddUninitialized( NumBytes );
	Ar.Serialize( SerializedData.GetData(), NumBytes );

	// Serialize the key times.
	Ar << Dst.Times;

	// Serialize mins and ranges.
	Ar << Dst.Mins[0] << Dst.Mins[1] << Dst.Mins[2];
	Ar << Dst.Ranges[0] << Dst.Ranges[1] << Dst.Ranges[2];
}

void UAnimSequence::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::Animation);
	
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);	

	FRawCurveTracks RawCurveCache;
	
	if (Ar.IsCooking())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RawCurveCache.FloatCurves = MoveTemp(RawCurveData.FloatCurves);
		RawCurveData.FloatCurves.Reset();

#if WITH_EDITORONLY_DATA
		RawCurveCache.TransformCurves = MoveTemp(RawCurveData.TransformCurves);
		RawCurveData.TransformCurves.Reset();
#endif
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UAnimSequence* StrippedRefPoseSeq = nullptr;
	if (Ar.IsCooking() && Ar.IsSaving())
	{
		if (UE::Anim::Private::ShouldStripAdditiveRefPose() && GetAdditiveAnimType() != EAdditiveAnimationType::AAT_None)
		{
			// Strip the additive base before UPROPERTY serialization in Super::Serialize
			StrippedRefPoseSeq = RefPoseSeq;
			RefPoseSeq = nullptr;
		}
	}

	Super::Serialize(Ar);

	if (StrippedRefPoseSeq)
	{
		// Restore after UPROPERTY serialization to avoid compression requests missing an additive base pose. 
		RefPoseSeq = StrippedRefPoseSeq;
	}

	if (Ar.IsCooking())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RawCurveData.FloatCurves = MoveTemp(RawCurveCache.FloatCurves);
#if WITH_EDITORONLY_DATA
		RawCurveData.TransformCurves = MoveTemp(RawCurveCache.TransformCurves);
#endif		
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FStripDataFlags StripFlags( Ar );
	if( !StripFlags.IsEditorDataStripped() )
	{
#if WITH_EDITORONLY_DATA
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Ar << RawAnimationData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (!Ar.IsCooking())
		{
			if (Ar.UEVer() >= VER_UE4_ANIMATION_ADD_TRACKCURVES)
			{
				if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RemovingSourceAnimationData)
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Ar << SourceRawAnimationData_DEPRECATED;

					if (SourceRawAnimationData_DEPRECATED.Num())
					{
						// Set RawAnimationData to Source
						RawAnimationData = SourceRawAnimationData_DEPRECATED;
						SourceRawAnimationData_DEPRECATED.Empty();
					}
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		}
#endif // WITH_EDITORONLY_DATA
	}

#if WITH_EDITOR
	if (Ar.IsCooking())
	{
		if (IsCompiling())
		{
			UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation({this});
		}
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	if ( Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::PerPlatformAnimSequenceTargetFrameRate)
	{
		PlatformTargetFrameRate = TargetFrameRate;
	}	
#endif // WITH_EDITORONLY_DATA

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::MoveCompressedAnimDataToTheDDC)
	{
		// Serialize the compressed byte stream from the archive to the buffer.
		int32 NumBytes;
		Ar << NumBytes;

		TArray<uint8> SerializedData;
		SerializedData.AddUninitialized(NumBytes);
		Ar.Serialize(SerializedData.GetData(), NumBytes);
	}
	else
	{
		const bool bIsCooking = Ar.IsCooking();
		const bool bIsDuplicating = Ar.HasAnyPortFlags(PPF_DuplicateForPIE) || Ar.HasAnyPortFlags(PPF_Duplicate);
		const bool bIsTransacting = Ar.IsTransacting();
		const bool bIsCookingWithoutAVData = bIsCooking && !Ar.CookingTarget()->AllowAudioVisualData();
		const bool bIsCountingMemory = Ar.IsCountingMemory();

		const bool bStripAnimDataOnDedicatedServer = StripAnimDataOnDedicatedServer == EStripAnimDataOnDedicatedServerSettings::UseProjectSetting ? UAnimationSettings::Get()->bStripAnimationDataOnDedicatedServer :
			StripAnimDataOnDedicatedServer == EStripAnimDataOnDedicatedServerSettings::StripAnimDataOnDedicatedServer;

		const bool bCookingTargetNeedsCompressedData = bIsCooking && (!bStripAnimDataOnDedicatedServer || !bIsCookingWithoutAVData || bEnableRootMotion);

		bool bSerializeCompressedData = bCookingTargetNeedsCompressedData || bIsDuplicating || bIsTransacting || bIsCountingMemory;
		Ar << bSerializeCompressedData;

		if (bCookingTargetNeedsCompressedData)
		{
			if(GetSkeleton())
			{
				// Validate that we are cooking valid compressed data.
				checkf(Ar.IsObjectReferenceCollector() || (GetSkeletonVirtualBoneGuid() == GetSkeleton()->GetVirtualBoneGuid()), TEXT("Attempting to cook animation '%s' containing invalid virtual bone guid! Animation:%s Skeleton:%s"), *GetFullName(), *GetSkeletonVirtualBoneGuid().ToString(EGuidFormats::HexValuesInBraces), *GetSkeleton()->GetVirtualBoneGuid().ToString(EGuidFormats::HexValuesInBraces));
			}
		}

		if (bSerializeCompressedData)
		{
			// Backwards compatibility, just serialize out compressed data
			if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimSequenceRawDataOnlyFlagRemoval)
			{
			    SerializeCompressedData(Ar,false);
			    if (!bIsTransacting)
			    {
					bool bTemp = false;
				    Ar << bTemp;
			    }
			}
			else
			{
#if WITH_EDITOR
				if (!GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
				{
					if (!bIsTransacting)
					{
						const bool bIsLoading = Ar.IsLoading();
						const bool bIsSaving = Ar.IsSaving();

						TUniquePtr<UE::TSharedLock<UE::FSharedRecursiveMutex>> SharedLock;

					    if (bIsLoading)
					    {
						    SharedCompressedDataMutex.Lock();
					    }
					    else
					    {
						    SharedLock = MakeUnique<UE::TSharedLock<UE::FSharedRecursiveMutex>>(SharedCompressedDataMutex);
					    }

						if (bCookingTargetNeedsCompressedData)
						{
							const ITargetPlatform* Platform = Ar.CookingTarget();
							checkf(Platform, TEXT("Invalid cooking target platform"));
							// During cook, serialize out singular (target platform) compressed data 
							FCompressedAnimSequence& PlatformCompressedData = GetPlatformCompressedData(Platform);
							check(PlatformCompressedData.IsValid(this, true));
							SerializeCompressedData(Ar, PlatformCompressedData);
						}
						else if (bIsDuplicating || bIsCountingMemory)
						{					
							// During duplication/transacting etc. serialize all contained compressed data
							if (bIsSaving)
							{
								uint32 NumPlatforms = DataByPlatformKeyHash.Num();
								Ar << NumPlatforms;

								FIoHash CurrentPlatformHash = FIoHash::Zero;
								for (TPair<FIoHash, TUniquePtr<FCompressedAnimSequence>>& CompressedDataPair : DataByPlatformKeyHash)
								{
									Ar << CompressedDataPair.Key;
									SerializeCompressedData(Ar, *CompressedDataPair.Value.Get());

									if (CurrentPlatformData && CurrentPlatformData == CompressedDataPair.Value.Get())
									{
										CurrentPlatformHash = CompressedDataPair.Key;
									}
								}
							
								// Store off the expected hash for CurrentPlatformData
								Ar << CurrentPlatformHash;
							}
							else
							{
								uint32 NumPlatforms = 0;
								Ar << NumPlatforms;
								for (uint32 Index = 0; Index < NumPlatforms; ++Index)
								{
									FIoHash PlatformHash;
									Ar << PlatformHash;
									FCompressedAnimSequence& TargetData = *DataByPlatformKeyHash.Emplace(PlatformHash, MakeUnique<FCompressedAnimSequence>()).Get();
									SerializeCompressedData(Ar, TargetData);
								}

								CurrentPlatformData = nullptr;
							
								// Try and restore short-circuit ptr for CurrentPlatformData according to its previously serialized hash
								FIoHash CurrentPlatformHash = FIoHash::Zero;
								Ar << CurrentPlatformHash;
								if (CurrentPlatformHash != FIoHash::Zero)
								{
									if (TUniquePtr<FCompressedAnimSequence>* Entry = DataByPlatformKeyHash.Find(CurrentPlatformHash))
									{
										CurrentPlatformData = Entry->Get();
									}
								}	
							}
						}

					    if (bIsLoading)
					    {
						    SharedCompressedDataMutex.Unlock();
					    }
					    else
					    {
						    SharedLock.Reset();
					    }
				    }
				    else
				    {
						UE_COMPRESSED_DATA_WRITE_SCOPE(this);
					    CurrentPlatformData = nullptr;
				    }
				}
				else
#endif
				{
					// During cooked runtime serialize in singular compressed data
					SerializeCompressedData(Ar, GetCompressedData_Internal());
				}
			}			
		}
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RenamingAnimationNumFrames)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NumberOfKeys = NumFrames;

		// Validate the actual number of keys that is stored, by looking at the maximum number of keys for any given animation track 
		int32 MaxNumberOfTrackKeys = 0;
		for (const FRawAnimSequenceTrack& Track : RawAnimationData)
		{
			MaxNumberOfTrackKeys = FMath::Max(MaxNumberOfTrackKeys, Track.PosKeys.Num());
			MaxNumberOfTrackKeys = FMath::Max(MaxNumberOfTrackKeys, Track.RotKeys.Num());
			MaxNumberOfTrackKeys = FMath::Max(MaxNumberOfTrackKeys, Track.ScaleKeys.Num());
		}

		const bool bUniformOrIdentityKeys = MaxNumberOfTrackKeys == 0 || MaxNumberOfTrackKeys == 1;
		// Test whether or not there are more track keys than the value stored, check for greater than or less than if the # of keys does not indicate single key or identity tracks
		if (MaxNumberOfTrackKeys > NumberOfKeys	|| (!bUniformOrIdentityKeys && MaxNumberOfTrackKeys < NumberOfKeys))
		{
			UE_LOGF(LogAnimation, Warning, "Animation %ls needs resaving - Invalid number of keys %i stored according to maximum number animation data track keys. Setting new number of keys %i.", *GetName(), NumberOfKeys, MaxNumberOfTrackKeys);
			NumberOfKeys = MaxNumberOfTrackKeys;
		}

		UpdateFrameRate_DeprecatedLoad();

		// In case there is any animation data available (not valid for curve only animations), verify that the new frame-rate matches up with the expected number of frames/keys and vice versa
		if (RawAnimationData.Num())
		{
			const int32 NumberOfFrames = FMath::Max(NumberOfKeys - 1, 1);
			const float SampledTime = SamplingFrameRate.AsSeconds(NumberOfFrames);

			// Validate that provided the number of keys and frame-rate the sequence length is accurate
			if (!FMath::IsNearlyEqual(SampledTime, SequenceLength, UE_KINDA_SMALL_NUMBER))
			{
				UE_LOGF(LogAnimation, Warning, "Animation %ls needs resaving - Inaccurate sequence length %5.5f according to number of frames and frame rate (delta of %f). Setting new length %5.5f.", *GetName(), SequenceLength, SampledTime - SequenceLength, SampledTime);
				SequenceLength = SampledTime;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if ( Ar.IsLoading() && Ar.UEVer() < VER_UE4_ASSET_IMPORT_DATA_AS_JSON && !AssetImportData)
	{
		// AssetImportData should always be valid
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}

	// SourceFilePath and SourceFileTimestamp were moved into a subobject
	if ( Ar.IsLoading() && Ar.UEVer() < VER_UE4_ADDED_FBX_ASSET_IMPORT_DATA && AssetImportData)
	{
		// AssetImportData should always have been set up in the constructor where this is relevant
		FAssetImportInfo Info;
		Info.Insert(FAssetImportInfo::FSourceFile(SourceFilePath_DEPRECATED));
		AssetImportData->SourceData = MoveTemp(Info);
		
		SourceFilePath_DEPRECATED = TEXT("");
		SourceFileTimestamp_DEPRECATED = TEXT("");
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
bool UAnimSequence::IsValidToPlay() const
{
	// make sure sequence length is valid and raw animation data exists, and compressed
	return (GetPlayLength() > 0.f);
}

void UAnimSequence::UpdateFrameRate_DeprecatedLoad()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const int32 NumberOfFrames = FMath::Max(NumberOfKeys - 1, 1);

	// Generate the frame-rate according to the number of frames and sequence length
	const double DecimalFrameRate = (double)NumberOfFrames / ((double)SequenceLength > 0.0 ? (double)SequenceLength : 1.0);

	// Account for non-whole number frame rates using large denominator
	const double Denominator = 1000000.0;
	SamplingFrameRate = FFrameRate(DecimalFrameRate * Denominator, Denominator);

	// Try to simplifiy the frame rate, in case it is a multiple of the commonly used frame rates e.g. 10000/300000 -> 1/30
	TArrayView<const FCommonFrameRateInfo> CommonFrameRates = FModuleManager::LoadModulePtr<ITimeManagementModule>("TimeManagement")->GetAllCommonFrameRates();
	for (const FCommonFrameRateInfo& Info : CommonFrameRates)
	{
		const bool bDoesNotAlreadyMatch = Info.FrameRate.Denominator != SamplingFrameRate.Denominator && Info.FrameRate.Numerator != SamplingFrameRate.Numerator;

		if (bDoesNotAlreadyMatch && FMath::IsNearlyEqual(SamplingFrameRate.AsInterval(), Info.FrameRate.AsInterval()))
		{
			SamplingFrameRate = Info.FrameRate;
			break;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif

void UAnimSequence::SortSyncMarkers()
{
	// First make sure all SyncMarkers are within a valid range
	for (auto& SyncMarker : AuthoredSyncMarkers)
	{
		SyncMarker.Time = FMath::Clamp(SyncMarker.Time, 0.f, GetPlayLength());
	}

	// Then sort
	AuthoredSyncMarkers.Sort();

	// Then refresh data
	RefreshSyncMarkerDataFromAuthored();
}

#if WITH_EDITOR

bool UAnimSequence::RemoveSyncMarkers(const TArray<FName>& NotifiesToRemove)
{
	bool bSequenceModified = false;
	for (int32 MarkerIndex = AuthoredSyncMarkers.Num() - 1; MarkerIndex >= 0; --MarkerIndex)
	{
		FAnimSyncMarker& Marker = AuthoredSyncMarkers[MarkerIndex];
		if (NotifiesToRemove.Contains(Marker.MarkerName))
		{
			if (!bSequenceModified)
			{
				Modify();
				bSequenceModified = true;
			}
			AuthoredSyncMarkers.RemoveAtSwap(MarkerIndex);
		}
	}

	if (bSequenceModified)
	{
		MarkPackageDirty();
		RefreshCacheData();
	}
	return bSequenceModified;
}

bool UAnimSequence::RenameSyncMarkers(FName InOldName, FName InNewName)
{
	bool bSequenceModified = false;
	for(FAnimSyncMarker& Marker : AuthoredSyncMarkers)
	{
		if(Marker.MarkerName == InOldName)
		{
			if (!bSequenceModified)
			{
				Modify();
				bSequenceModified = true;
			}

			Marker.MarkerName = InNewName;
		}
	}

	if (bSequenceModified)
	{
		MarkPackageDirty();
		RefreshCacheData();
	}
	return bSequenceModified;
}

#endif

void UAnimSequence::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	// We preload the compression settings because we need them loaded during Serialize to lookup the proper codec
	// which is stored as a path/string.
	if (CurveCompressionSettings != nullptr)
	{
		OutDeps.Add(CurveCompressionSettings);
	}

	if (BoneCompressionSettings != nullptr)
	{
		OutDeps.Add(BoneCompressionSettings);
	}
	if (VariableFrameStrippingSettings != nullptr)
	{
		OutDeps.Add(VariableFrameStrippingSettings);
	}
}

#if WITH_EDITOR
class FAnimSequenceDeterminismHelper : public UE::Cook::IDeterminismHelper
{
public:
	FAnimSequenceDeterminismHelper(UAnimSequence* InAnimSequence)
		: AnimSequence(InAnimSequence)
	{
	}

	virtual void ConstructDiagnostics(UE::Cook::IDeterminismConstructDiagnosticsContext& Context) override
	{
		FString DDCKey = AnimSequence->CreateDerivedDataKeyString(Context.GetTargetPlatform());
		FIoHash DDCKeyHash = FIoHash::Zero;
		const IAnimationDataModel* Model = AnimSequence->DataModelInterface.GetInterface();
		FString ModelStateString = Model ? Model->GenerateDebugStateString() : TEXT("NoDataModel");
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		bool bRawDataGuidEqual = Model ? (AnimSequence->RawDataGuid == Model->GenerateGuid()): false;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;

		if (!DDCKey.IsEmpty())
		{
			FMemoryHasherBlake3 Writer;
			Writer << DDCKey;
			DDCKeyHash = Writer.Finalize();
		}

		FCbWriter Writer;
		Writer.BeginObject();
		Writer << "DDCKeyHash" << WriteToString<64>(DDCKeyHash);
		if (Context.FullDDCKeysRequested())
		{
			Writer << "DDCKey" << DDCKey;
		}
		//Limiting the diagnostic size to the first 1000 chars
		Writer << "Model" << FStringView(ModelStateString).Left(1000);
		Writer << "RawDataGuidEqual" << bRawDataGuidEqual;
		Writer.EndObject();

		Context.AddDiagnostic("UAnimSequence", Writer.Save());
	}

private:
	UAnimSequence* AnimSequence;
};
#endif

void UAnimSequence::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	if (ObjectSaveContext.IsCooking())
	{
		check(IsCachedCookedPlatformDataLoaded(ObjectSaveContext.GetTargetPlatform()));

		// In case compressed data was cleared between ::IsCachedCookedPlatformDataLoaded and ::PreSave being called, synchronously run compression
		if(!IsCachedCookedPlatformDataLoaded(ObjectSaveContext.GetTargetPlatform()))
		{
			CacheDerivedDataForPlatform(ObjectSaveContext.GetTargetPlatform());
			check(IsCachedCookedPlatformDataLoaded(ObjectSaveContext.GetTargetPlatform()));
		}
	}

	if (!ObjectSaveContext.IsProceduralSave())
	{
		UpdateRetargetSourceAssetData();
	}

	if (ObjectSaveContext.IsDeterminismDebug())
	{
		ObjectSaveContext.RegisterDeterminismHelper(new FAnimSequenceDeterminismHelper(this));
	}

	// Updated current platform cache data ptr during non-cook editor runtime
	if (!ObjectSaveContext.IsCooking())
	{
		// Find compressed data for current platform and cache its ptr
		const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		if(HasCompressedDataForPlatform(RunningPlatform))
		{
			UE_COMPRESSED_DATA_WRITE_SCOPE(this);
			CurrentPlatformData = &GetPlatformCompressedData(RunningPlatform);
		}
	}
#endif

	Super::PreSave(ObjectSaveContext);
}

void UAnimSequence::PostLoad()
{
#if WITH_EDITOR
	ValidateCompressionSettings();
#endif // WITH_EDITOR
	
	//Parent PostLoad will ensure that skeleton is fully loaded
	//before we do anything further in PostLoad
	Super::PostLoad();

#if WITH_EDITOR
	if (RefPoseSeq && RefPoseSeq->HasAnyFlags(EObjectFlags::RF_NeedPostLoad))
	{
		RefPoseSeq->ConditionalPostLoad();
	}
#endif // WITH_EDITOR

	// if valid additive, but if base additive isn't 
	// this seems to happen from retargeting sometimes, which we still have to investigate why, 
	// but this causes issue since once this happens this is unrecoverable until you delete from outside of editor
	if (IsValidAdditive())
	{
		if (RefPoseSeq && RefPoseSeq->GetSkeleton() == nullptr)
		{
			// if this happens, there was a issue with retargeting
			UE_LOGF(LogAnimation, Warning, "Animation %ls - Invalid additive animation base animation (%ls)", *GetName(), *RefPoseSeq->GetName());
			RefPoseSeq = nullptr;
		}
	}

	// Ensure notifies are sorted.
	SortNotifies();

	// No animation data is found. Warn - this should check before we check CompressedTrackOffsets size
	// Otherwise, we'll see empty data set crashing game due to no CompressedTrackOffsets
	// You can't check RawAnimationData size since it gets removed during cooking
	if (GetOutermost()->HasAnyPackageFlags(PKG_Cooked | PKG_FilterEditorOnly))
	{
		// Update the virtual bone GUID according to the skeleton
		if (const USkeleton* CurrentSkeleton = GetSkeleton())
		{
			SetSkeletonVirtualBoneGuid(CurrentSkeleton->GetVirtualBoneGuid());

			if(!IsCompressedDataValid())
			{
				UE_LOGF(LogAnimation, Fatal, "No animation compression exists for sequence %ls (%ls)", *GetName(), (GetOuter() ? *GetOuter()->GetFullName() : *GetFullName()) );
			}
		}
	}

	for(FAnimNotifyEvent& Notify : Notifies)
	{
#if WITH_EDITORONLY_DATA
		if(Notify.DisplayTime_DEPRECATED != 0.0f)
		{
			Notify.Clear();
			Notify.Link(this, Notify.DisplayTime_DEPRECATED);
		}
		else
#endif
		{
			Notify.Link(this, Notify.GetTime());
		}
	
		if(Notify.Duration != 0.0f)
		{
			Notify.EndLink.Link(this, Notify.GetTime() + Notify.Duration);
		}
	}

	if (USkeleton* CurrentSkeleton = GetSkeleton())
	{
#if WITH_EDITOR
		for (const FAnimSyncMarker& SyncMarker : AuthoredSyncMarkers)
		{
			CurrentSkeleton->RegisterMarkerName(SyncMarker.MarkerName);
		}
#endif
	}
}

#if WITH_EDITOR
void ShowResaveMessage(const UAnimSequence* Sequence)
{
	if (IsRunningCommandlet())
	{
		UE_LOGF(LogAnimation, Log, "Resave Animation Required(%ls, %ls): Fixing track data and recompressing.", *GetNameSafe(Sequence), *Sequence->GetPathName());

		static FName NAME_LoadErrors("LoadErrors");
		FMessageLog LoadErrors(NAME_LoadErrors);

		TSharedRef<FTokenizedMessage> Message = LoadErrors.Warning();
		Message->AddToken(FTextToken::Create(LOCTEXT("AnimationNeedsResave1", "The Animation ")));
		Message->AddToken(FAssetNameToken::Create(Sequence->GetPathName(), FText::FromString(GetNameSafe(Sequence))));
		Message->AddToken(FTextToken::Create(LOCTEXT("AnimationNeedsResave2", " needs resave.")));
		LoadErrors.Notify();
	}
}

EDataValidationResult UAnimSequence::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = Super::IsDataValid(Context);
	// Do not validate cooked anim sequence
	if (GetPackage()->HasAnyPackageFlags(PKG_Cooked) == false)
	{
		if (!GetSkeleton())
		{
			Context.AddError(LOCTEXT("AnimSequenceValidation_NoSkeleton", "This anim sequence asset has no Skeleton. Anim sequence asset need a valid skeleton."));
			ValidationResult = EDataValidationResult::Invalid;
		}
	}
	return ValidationResult;
}

#endif // WITH_EDITOR
void UAnimSequence::BeginDestroy()
{
#if WITH_EDITOR
	// Could already be compressing
	WaitOnExistingCompression(false);
#endif // WITH_EDITOR

	Super::BeginDestroy();

#if !WITH_EDITOR
	GetCompressedData_Internal().Reset();
#endif
}

bool UAnimSequence::IsReadyForFinishDestroy()
{
	if (!Super::IsReadyForFinishDestroy())
	{
		return false;
	}

#if WITH_EDITOR
	WaitOnExistingCompression(false);
#endif

	return true;
}

#if WITH_EDITOR
void UAnimSequence::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(!IsTemplate() && !FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		// Make sure package is marked dirty when doing stuff like adding/removing notifies
		MarkPackageDirty();
	}

	if (AdditiveAnimType != AAT_None)
	{
		if (RefPoseType == ABPT_None)
		{
			// slate will take care of change
			RefPoseType = ABPT_RefPose;
		}
	}

	if (RefPoseSeq != nullptr)
	{
		if (RefPoseSeq->GetSkeleton() != GetSkeleton()) // @todo this may require to be changed when hierarchy of skeletons is introduced
		{
			RefPoseSeq = nullptr;
		}
	}

	bool bAdditiveSettingsChanged = false;
	bool bCompressionAffectingSettingsChanged = false;
	bool bShouldResample = false;

	if(PropertyChangedEvent.Property)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, RetargetSourceAsset))
		{
			UpdateRetargetSourceAssetData();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		const bool bChangedRefFrameIndex = PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, RefFrameIndex);

		if ((bChangedRefFrameIndex && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, RefPoseSeq) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, RefPoseType))
		{
			bAdditiveSettingsChanged = true;

			if (RefPoseType == ABPT_RefPose || RefPoseType == ABPT_LocalAnimFrame || AdditiveAnimType == AAT_None)
			{
				// clear RefPoseSeq when selecting settings which will hide the field
				RefPoseSeq = nullptr;
			}
		}
		
		bCompressionAffectingSettingsChanged =   PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, bAllowFrameStripping)
											  || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, CompressionErrorThresholdScale)
											  || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, BoneCompressionSettings)
											  || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, CurveCompressionSettings)
											  || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, VariableFrameStrippingSettings);

		bShouldResample = PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAnimSequence, PlatformTargetFrameRate) || bChangedRefFrameIndex;
	}

	if (bShouldResample)
	{
		CalculateNumberOfSampledKeys();
	}

	const bool bNeedPostProcess = !IsCompressedDataValid() || bAdditiveSettingsChanged || bCompressionAffectingSettingsChanged || bShouldResample;

	// @Todo fix me: This is temporary fix to make sure they always have compressed data
	ValidateModel();
	if (IsDataModelValid() && bNeedPostProcess)
	{		
		ClearAllCompressionData();
		BeginCacheDerivedDataForCurrentPlatform();
	}
}

void UAnimSequence::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	BeginCacheDerivedData(TargetPlatform);
}
#endif // WITH_EDITOR

#if WITH_EDITOR

bool UAnimSequence::HasCompressedDataForHash(const FIoHash& InKeyHash) const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this)
	
	if (InKeyHash.IsZero())
	{
		UE_LOGF(LogAnimation, Warning, "HasCompressedDataForHash: Zero key hash %ls ", *GetName());
		return false;
	}
		
	if(const TUniquePtr<FCompressedAnimSequence>* CompressedDataPtr = DataByPlatformKeyHash.Find(InKeyHash))
	{
		if ((*CompressedDataPtr)->IsValid(this))
		{
			UE::TScopeLock LockGuard(CacheTasksMutex);
			if (CacheTasksByKeyHash.Contains(InKeyHash))
			{
				return false;
			}

			return true;
		}
	}

	return false;
}

bool UAnimSequence::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	const FIoHash KeyHash = GetDerivedDataKeyHash(TargetPlatform);
	if (KeyHash.IsZero())
	{
		UE_LOGF(LogAnimation, Warning, "Zero key hash compressed animation data for %ls requested platform %ls", *GetName(), *TargetPlatform->PlatformName());
		return true;
	}

	if (PollCacheDerivedData(KeyHash))
	{
		EndCacheDerivedData(KeyHash);
	}

	bool bShouldRecompress = false;
	{
		UE_COMPRESSED_DATA_READ_SCOPE(this);
		UE::TScopeLock LockGuard(CacheTasksMutex);
		if(const TUniquePtr<FCompressedAnimSequence>* CompressedDataPtr = DataByPlatformKeyHash.Find(KeyHash))
		{
			if (!CacheTasksByKeyHash.Contains(KeyHash) && (*CompressedDataPtr)->IsValid(this))
			{
				return true;
			}
			else if (!CacheTasksByKeyHash.Contains(KeyHash))
			{
				bShouldRecompress = true;
			}
		}
		else if (!CacheTasksByKeyHash.Contains(KeyHash))
		{
			bShouldRecompress = true;
		}
	}

	if (bShouldRecompress)
	{
		if (!HasAnyFlags(EObjectFlags::RF_NeedPostLoad) && CanBeCompressed())
		{
			BeginCacheForCookedPlatformData(TargetPlatform);
		}
	}

	return false;
}
#endif // WITH_EDITOR

// @todo DB: Optimize!
template<typename TimeArray>
static int32 FindKeyIndex(float Time, const TimeArray& Times)
{
	int32 FoundIndex = 0;
	for ( int32 Index = 0 ; Index < Times.Num() ; ++Index )
	{
		const float KeyTime = Times(Index);
		if ( Time >= KeyTime )
		{
			FoundIndex = Index;
		}
		else
		{
			break;
		}
	}
	return FoundIndex;
}

#if WITH_EDITOR
int32 GForceRawData = 0;
static FAutoConsoleVariableRef CVarForceRawData(
	TEXT("a.ForceEvalRawData"),
	GForceRawData,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether or not to forcefully sample non-compressed anim data."),
	ECVF_Cheat);
#endif // WITH_EDITOR

void UAnimSequence::GetBoneTransform(FTransform& OutAtom, FSkeletonPoseBoneIndex BoneIndex, const FAnimExtractContext& ExtractionContext, bool bUseRawData) const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	GetBoneTransform_Lockless(OutAtom, BoneIndex, ExtractionContext, bUseRawData);	
}

void UAnimSequence::GetBoneTransform_Lockless(FTransform& OutAtom, FSkeletonPoseBoneIndex BoneIndex, const FAnimExtractContext& ExtractionContext, bool bForceUseRawData) const
{	
	const FCompressedAnimSequence& PlatformCompressedData = GetPlatformCompressedData(ExtractionContext);
	const bool bUseRawDataForPoseExtraction = (CanEvaluateRawAnimationData() && bForceUseRawData) || ShouldUseRawDataForPoseExtraction_Lockless(ExtractionContext);

#if WITH_EDITOR
	ensure(!ExtractionContext.bEnforceCompressedDataSampling || (PlatformCompressedData.IsValid(this) && !bUseRawDataForPoseExtraction));
#endif // WITH_EDITOR

	const EAnimInterpolationType InterpolationType = ExtractionContext.InterpolationOverride.Get(Interpolation);
	if (!bUseRawDataForPoseExtraction && PlatformCompressedData.IsValid(this))
	{
		FAnimSequenceDecompressionContext DecompContext(PlatformTargetFrameRate.Default, PlatformTargetFrameRate.Default.AsFrameTime(GetPlayLength()).RoundToFrame().Value, InterpolationType, GetRetargetTransformsSourceName(), *PlatformCompressedData.CompressedDataStructure, GetSkeleton()->GetRefLocalPoses(), PlatformCompressedData.CompressedTrackToSkeletonMapTable, GetSkeleton(), IsValidAdditive(), AdditiveAnimType);
		DecompContext.Seek(ExtractionContext.CurrentTime);
		if (PlatformCompressedData.BoneCompressionCodec)
		{
			const int32 TrackIndex = PlatformCompressedData.GetTrackIndexFromSkeletonIndex(BoneIndex.GetInt());
			if (TrackIndex != INDEX_NONE)
			{				
				PlatformCompressedData.BoneCompressionCodec->DecompressBone(DecompContext, TrackIndex, OutAtom);
			}
		}
	}
	else if (CanEvaluateRawAnimationData())
	{
#if WITH_EDITOR
		ValidateModel();
		const FName BoneName = GetSkeleton()->GetReferenceSkeleton().GetBoneName(BoneIndex.GetInt());
		OutAtom = DataModelInterface->EvaluateBoneTrackTransform(BoneName, DataModelInterface->GetFrameRate().AsFrameTime(ExtractionContext.CurrentTime), InterpolationType);

		const FAnimationCurveIdentifier TransformCurveId(BoneName, ERawCurveTrackTypes::RCT_Transform);
		if (const FTransformCurve* TransformCurvePtr = DataModelInterface->FindTransformCurve(TransformCurveId))
		{
			const FTransform AdditiveTransform = TransformCurvePtr->Evaluate(ExtractionContext.CurrentTime, 1.f);
			const FTransform LocalTransform = OutAtom;
			OutAtom.SetRotation(LocalTransform.GetRotation() * AdditiveTransform.GetRotation());
			OutAtom.SetTranslation(LocalTransform.TransformPosition(AdditiveTransform.GetTranslation()));
			OutAtom.SetScale3D(LocalTransform.GetScale3D() * AdditiveTransform.GetScale3D());	
		}
#endif
	}
}

void UAnimSequence::HandleAssetPlayerTickedInternal(FAnimAssetTickContext &Context, const float PreviousTime, const float MoveDelta, const FAnimTickRecord &Instance, struct FAnimNotifyQueue& NotifyQueue) const
{
	Super::HandleAssetPlayerTickedInternal(Context, PreviousTime, MoveDelta, Instance, NotifyQueue);

	if (bEnableRootMotion)
	{
		UE_COMPRESSED_DATA_READ_SCOPE(this);
		const FAnimExtractContext ExtractionContext(static_cast<double>(PreviousTime), true, FDeltaTimeRecord(MoveDelta), Instance.bLooping);
		Context.RootMotionMovementParams.Accumulate(ExtractRootMotion(ExtractionContext));
	}
}

FTransform UAnimSequence::ExtractRootTrackTransform(const FAnimExtractContext& ExtractionContext, const FBoneContainer* RequiredBones) const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	return ExtractRootTrackTransform_Lockless(ExtractionContext, RequiredBones);
}

FTransform UAnimSequence::ExtractRootMotion(const FAnimExtractContext& ExtractionContext) const
{
	FRootMotionMovementParams RootMotionParams;

	if (ExtractionContext.DeltaTimeRecord.Delta != 0.f)
	{
		const bool bPlayingBackwards = (ExtractionContext.DeltaTimeRecord.Delta < 0.f);

		float PreviousPosition = ExtractionContext.CurrentTime;
		float CurrentPosition = ExtractionContext.CurrentTime;
		float DesiredDeltaMove = ExtractionContext.DeltaTimeRecord.Delta;

		do
		{
			// Disable looping here. Advance to desired position, or beginning / end of animation 
			const ETypeAdvanceAnim AdvanceType = FAnimationRuntime::AdvanceTime(false, DesiredDeltaMove, CurrentPosition, GetPlayLength());

			// Verify position assumptions
			ensureMsgf(bPlayingBackwards ? (CurrentPosition <= PreviousPosition) : (CurrentPosition >= PreviousPosition), TEXT("in Animation %s(Skeleton %s) : bPlayingBackwards(%d), PreviousPosition(%0.2f), Current Position(%0.2f)"),
				*GetName(), *GetNameSafe(GetSkeleton()), bPlayingBackwards, PreviousPosition, CurrentPosition);

			RootMotionParams.Accumulate(ExtractRootMotionFromRange(PreviousPosition, CurrentPosition, ExtractionContext));

			// If we've hit the end of the animation, and we're allowed to loop, keep going.
			if ((AdvanceType == ETAA_Finished) && ExtractionContext.bLooping)
			{
				const double ActualDeltaMove = (CurrentPosition - PreviousPosition);
				DesiredDeltaMove -= ActualDeltaMove;

				PreviousPosition = bPlayingBackwards ? GetPlayLength() : 0.f;
				CurrentPosition = PreviousPosition;
			}
			else
			{
				break;
			}
		} while (true);
	}

	return RootMotionParams.GetRootMotionTransform();
}

FTransform UAnimSequence::ExtractRootMotionFromRange(double StartTime, double EndTime, const FAnimExtractContext& ExtractionContext) const
{
	const FVector DefaultScale(1.f);

	FTransform RootTransformRefPose = FTransform::Identity;
	if (const USkeleton* MySkeleton = GetSkeleton())
	{
		const FReferenceSkeleton& RefSkeleton = MySkeleton->GetReferenceSkeleton();
		if (RefSkeleton.GetNum() > 0)
		{
			RootTransformRefPose = RefSkeleton.GetRefBonePose()[0];
		}
	}
	else
	{
		UE_LOGF(LogAnimation, Error, "ExtractRootMotionFromRange for %ls failed, Skeleton == nullptr.", *GetName());
		return FTransform::Identity;
	}

	UE_COMPRESSED_DATA_READ_SCOPE(this);
	FAnimExtractContext Context = ExtractionContext;
	Context.CurrentTime = StartTime;
	FTransform StartTransform = ExtractRootTrackTransform_Lockless(Context, nullptr);
	Context.CurrentTime = EndTime;
	FTransform EndTransform = ExtractRootTrackTransform_Lockless(Context, nullptr);

	// Use old calculation if needed.
	if (bUseNormalizedRootMotionScale)
	{
		//Clear scale as it will muck up GetRelativeTransform
		StartTransform.SetScale3D(FVector(1.f));
		EndTransform.SetScale3D(FVector(1.f));
	}
	else
	{
		if (IsValidAdditive())
		{
			StartTransform.SetScale3D(StartTransform.GetScale3D() + DefaultScale);
			EndTransform.SetScale3D(EndTransform.GetScale3D() + DefaultScale);
		}
	}

	// Transform to Component Space
	const FTransform RootToComponent = RootTransformRefPose.Inverse();
	StartTransform = RootToComponent * StartTransform;
	EndTransform = RootToComponent * EndTransform;

	return EndTransform.GetRelativeTransform(StartTransform);
}

#if WITH_EDITOR
TArray<const UAnimSequence*> CurrentBakingAnims;
#endif

#define DEBUG_POSE_OUTPUT 0

#if DEBUG_POSE_OUTPUT
void DebugPrintBone(const FCompactPose& OutPose, const FCompactPoseBoneIndex& BoneIndex, int32 OutIndent)
{
	for (int i = 0; i < OutIndent; ++i)
	{
		FPlatformMisc::LowLevelOutputDebugString(TEXT("  "));
	}
	const FBoneContainer& Cont = OutPose.GetBoneContainer();

	FName BoneName = Cont.GetReferenceSkeleton().GetBoneName(Cont.MakeMeshPoseIndex(BoneIndex).GetInt());

	FVector T = OutPose[BoneIndex].GetTranslation();

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s - (%.2f, %.2f,%.2f)\n"), *BoneName.ToString(), T.X, T.Y, T.Z);
}
#endif

void UAnimSequence::GetAnimationPose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const
{
	SCOPE_CYCLE_COUNTER(STAT_GetAnimationPose);

	const FCompactPose& OutPose = OutAnimationPoseData.GetPose();

	// @todo anim: if compressed and baked in the future, we don't have to do this 
	if (IsValidAdditive() && ShouldUseRawDataForPoseExtraction(OutPose.GetBoneContainer(), ExtractionContext))
	{
		if (AdditiveAnimType == AAT_LocalSpaceBase)
		{
			GetBonePose_Additive(OutAnimationPoseData, ExtractionContext);
		}
		else if (AdditiveAnimType == AAT_RotationOffsetMeshSpace)
		{
			GetBonePose_AdditiveMeshRotationOnly(OutAnimationPoseData, ExtractionContext);
		}
	}
	else
	{
		GetBonePose(OutAnimationPoseData, ExtractionContext);
	}

	// If the sequence has root motion enabled, allow sampling of a root motion delta into the custom attribute container of the outgoing pose
	if (HasRootMotion()
#if WITH_EDITOR
		&& ExtractionContext.bExtractWithRootMotionProvider
#endif // WITH_EDITOR
		)
	{
		if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
		{
			RootMotionProvider->SampleRootMotion(ExtractionContext.DeltaTimeRecord, *this, ExtractionContext.bLooping, OutAnimationPoseData.GetAttributes());
		}
	}

	// Check that all bone atoms coming from animation are normalized
#if DO_CHECK && WITH_EDITORONLY_DATA
	check(OutPose.IsNormalized());
#endif

#if DEBUG_POSE_OUTPUT
	TArray<TArray<int32>> ParentLevel;
	ParentLevel.Reserve(64);
	for (int32 i = 0; i < 64; ++i)
	{
		ParentLevel.Add(TArray<int32>());
	}
	ParentLevel[0].Add(0);

	FPlatformMisc::LowLevelOutputDebugString(TEXT("\nGetAnimationPose\n"));
	
	DebugPrintBone(OutPose, FCompactPoseBoneIndex(0), 0);
	for (FCompactPoseBoneIndex BoneIndex(1); BoneIndex < OutPose.GetNumBones(); ++BoneIndex)
	{
		FCompactPoseBoneIndex ParentIndex = OutPose.GetBoneContainer().GetParentBoneIndex(BoneIndex);
		int32 Indent = 0;
		for (; Indent < ParentLevel.Num(); ++Indent)
		{
			if (ParentLevel[Indent].Contains(ParentIndex.GetInt()))
			{
				break;
			}
		}
		Indent += 1;
		check(Indent < 64);
		ParentLevel[Indent].Add(BoneIndex.GetInt());

		DebugPrintBone(OutPose, BoneIndex, Indent);
	}
#endif
}

void UAnimSequence::GetBonePose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext, bool bForceUseRawData /*= false*/) const
{
	SCOPE_CYCLE_COUNTER(STAT_AnimSeq_GetBonePose);
	CSV_SCOPED_TIMING_STAT(Animation, AnimSeq_GetBonePose);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();

	FBoneContainer& RequiredBones = OutPose.GetBoneContainer();
	check(!bForceUseRawData || CanEvaluateRawAnimationData());

	UE_COMPRESSED_DATA_READ_SCOPE(this);
	const USkeleton* MySkeleton = GetSkeleton();
	const bool bInvalidSkeleton = MySkeleton == nullptr;
	const bool bUseRawDataForPoseExtraction = bInvalidSkeleton || (CanEvaluateRawAnimationData() && bForceUseRawData) || ShouldUseRawDataForPoseExtraction_Lockless(RequiredBones, ExtractionContext);

	const bool bIsBakedAdditive = (!bUseRawDataForPoseExtraction || bInvalidSkeleton) && IsValidAdditive();
	if (bInvalidSkeleton)
	{
		if (bIsBakedAdditive)
		{
			OutPose.ResetToAdditiveIdentity();
		}
		else
		{
			OutPose.ResetToRefPose();
		}
		return;
	}
	
	const FCompressedAnimSequence& PlatformCompressedData = GetPlatformCompressedData(ExtractionContext);
#if WITH_EDITOR
	check(!ExtractionContext.bEnforceCompressedDataSampling || (PlatformCompressedData.IsValid(this) && !bUseRawDataForPoseExtraction));
#endif // WITH_EDITOR

	const bool bDisableRetargeting = RequiredBones.GetDisableRetargeting();

	// initialize with ref-pose
	if (bIsBakedAdditive)
	{
		//When using baked additive ref pose is identity
		OutPose.ResetToAdditiveIdentity();
	}
	else
	{
		// if retargeting is disabled, we initialize pose with 'Retargeting Source' ref pose.
		if (bDisableRetargeting)
		{
			TArray<FTransform> const& AuthoredOnRefSkeleton = GetRetargetTransforms();
			const TArray<FTransform>& SkeletonRefPose = RequiredBones.GetSkeletonAsset()->GetRefLocalPoses();

			// Map from this sequence its Skeleton to target
			const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(MySkeleton, RequiredBones.GetSkeletonAsset());
			for (FCompactPoseBoneIndex PoseBoneIndex : OutPose.ForEachBoneIndex())
			{
				const int32 SkeletonBoneIndex = RequiredBones.GetSkeletonPoseIndexFromCompactPoseIndex(PoseBoneIndex).GetInt();
				const int32 RemappedSkeletonBoneIndex = SkeletonRemapping.IsValid() ? SkeletonRemapping.GetSourceSkeletonBoneIndex(SkeletonBoneIndex) : SkeletonBoneIndex;

				// Virtual bones are part of the retarget transform pose, so if the pose has not been updated (recently) there might be a mismatch
				if (RemappedSkeletonBoneIndex != INDEX_NONE && AuthoredOnRefSkeleton.IsValidIndex(RemappedSkeletonBoneIndex))
				{
					OutPose[PoseBoneIndex] = AuthoredOnRefSkeleton[RemappedSkeletonBoneIndex];
				}
				else
				{
					// Only do this if we have a mesh. otherwise we're not retargeting animations.
					if (RequiredBones.GetSkeletalMeshAsset())
					{
						// Pose bone index should always exist in Skeleton
						checkSlow(SkeletonBoneIndex != INDEX_NONE);
						OutPose[PoseBoneIndex] = SkeletonRefPose[SkeletonBoneIndex];
					}
					else
					{
						OutPose[PoseBoneIndex] = RequiredBones.GetRefPoseTransform(PoseBoneIndex);
					}
				}
			}
		}
		else
		{
			OutPose.ResetToRefPose();
		}
	}

#if WITH_EDITOR
	const int32 NumTracks =	bUseRawDataForPoseExtraction ? DataModelInterface->GetNumBoneTracks() : PlatformCompressedData.CompressedTrackToSkeletonMapTable.Num();
#else
	const int32 NumTracks =	PlatformCompressedData.CompressedTrackToSkeletonMapTable.Num();
#endif

	const bool bTreatAnimAsAdditive = (IsValidAdditive() && !bUseRawDataForPoseExtraction); // Raw data is never additive
	FAnimExtractContext RootMotionContext = ExtractionContext;
	RootMotionContext.CurrentTime = 0.0;
	const FRootMotionReset RootMotionReset(bEnableRootMotion, RootMotionRootLock,
#if WITH_EDITOR
		!ExtractionContext.bIgnoreRootLock &&
#endif // WITH_EDITOR
		bForceRootLock,
		ExtractRootTrackTransform_Lockless(RootMotionContext, &RequiredBones), bTreatAnimAsAdditive);

#if WITH_EDITOR
	// Evaluate raw (source) curve and bone data
	if (bUseRawDataForPoseExtraction)
	{
		{
			const EAnimInterpolationType InterpolationType = ExtractionContext.InterpolationOverride.Get(Interpolation);
			const UE::Anim::DataModel::FEvaluationContext EvaluationContext(ExtractionContext.CurrentTime, DataModelInterface->GetFrameRate(), GetRetargetTransformsSourceName(), GetRetargetTransforms(), InterpolationType);
			DataModelInterface->Evaluate(OutAnimationPoseData, EvaluationContext);
		}

		if ((ExtractionContext.bExtractRootMotion && RootMotionReset.bEnableRootMotion) || RootMotionReset.bForceRootLock)
		{
			RootMotionReset.ResetRootBoneForRootMotion(OutPose[FCompactPoseBoneIndex(0)], RequiredBones);
		}

		return;
	}
	else
#endif // WITH_EDITOR
	// Only try and evaluate compressed bone data if the animation contains any bone tracks
	if (NumTracks != 0)
	{
		// Evaluate compressed bone data
		const EAnimInterpolationType InterpolationType = ExtractionContext.InterpolationOverride.Get(Interpolation);
		FAnimSequenceDecompressionContext DecompContext(PlatformTargetFrameRate.Default, PlatformTargetFrameRate.Default.AsFrameTime(GetPlayLength()).RoundToFrame().Value, InterpolationType, GetRetargetTransformsSourceName(), *PlatformCompressedData.CompressedDataStructure, GetSkeleton()->GetRefLocalPoses(), PlatformCompressedData.CompressedTrackToSkeletonMapTable, GetSkeleton(), IsValidAdditive(), AdditiveAnimType);
		UE::Anim::Decompression::DecompressPose(OutPose, PlatformCompressedData, ExtractionContext, DecompContext, GetRetargetTransforms(), RootMotionReset);
	}

	// (Always) evaluate compressed curve data
	{
#if WITH_EDITOR
		// When evaluating from raw animation data, UE::Anim::BuildPoseFromModel will populate the curve data
		if (!bUseRawDataForPoseExtraction)
#endif // WITH_EDITOR
		{
			EvaluateCurveData_Lockless(OutAnimationPoseData.GetCurve(), ExtractionContext, bUseRawDataForPoseExtraction);
		}
	}

	// Evaluate animation attributes (no compressed format yet)
	EvaluateAttributes(OutAnimationPoseData, ExtractionContext, false);
}

void UAnimSequence::GetBonePose_Additive(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const
{
	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	if (!IsValidAdditive())
	{
		OutPose.ResetToAdditiveIdentity();
		return;
	}

	// Extract target pose
	GetBonePose(OutAnimationPoseData, ExtractionContext);

	// Extract base pose
	FCompactPose BasePose;
	FBlendedCurve BaseCurve;
	UE::Anim::FStackAttributeContainer BaseAttributes;
	
	BasePose.SetBoneContainer(&OutPose.GetBoneContainer());
	BaseCurve.InitFrom(OutCurve);	

	FAnimationPoseData BasePoseData(BasePose, BaseCurve, BaseAttributes);

	GetAdditiveBasePose(BasePoseData, ExtractionContext);

	// Create Additive animation
	FAnimationRuntime::ConvertPoseToAdditive(OutPose, BasePose);
	OutCurve.ConvertToAdditive(BaseCurve);

	UE::Anim::Attributes::ConvertToAdditive(BaseAttributes, OutAttributes);
}

static void GetSequencePose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext, const UAnimSequence &Seq, int32 FrameIndex, bool bForceUseRawData)
{
	const double Fraction = (Seq.GetNumberOfSampledKeys() > 0) ? FMath::Clamp<double>((double)FrameIndex / (double)Seq.GetNumberOfSampledKeys(), 0.0, 1.0) : 0.0;
	const double BasePoseTime = Seq.GetPlayLength() * Fraction;
	FAnimExtractContext BasePoseExtractionContext(ExtractionContext);
	BasePoseExtractionContext.CurrentTime = BasePoseTime;
	Seq.GetBonePose(OutAnimationPoseData, BasePoseExtractionContext, bForceUseRawData);
}

void UAnimSequence::GetAdditiveBasePose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const
{
	switch (RefPoseType)
	{
		// use whole animation as a base pose. Need BasePoseSeq.
		case ABPT_AnimScaled:
		{
			// normalize time to fit base seq
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			const double Fraction = (GetPlayLength() > 0.0)? FMath::Clamp<double>(ExtractionContext.CurrentTime / (double)GetPlayLength(), 0.0, 1.0) : 0.0;
			const double BasePoseTime = (double)RefPoseSeq->GetPlayLength() * Fraction;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			FAnimExtractContext BasePoseExtractionContext(ExtractionContext);
			BasePoseExtractionContext.CurrentTime = BasePoseTime;
			RefPoseSeq->GetBonePose(OutAnimationPoseData, BasePoseExtractionContext, CanEvaluateRawAnimationData());
			break;
		}
		// use animation as a base pose. Need BasePoseSeq and RefFrameIndex (will clamp if outside).
		case ABPT_AnimFrame:
		{
			GetSequencePose(OutAnimationPoseData, ExtractionContext, *RefPoseSeq, RefFrameIndex, CanEvaluateRawAnimationData());
			break;
		}
		// use this animation as a base pose. Need RefFrameIndex (will clamp if outside).
		case ABPT_LocalAnimFrame:
		{
			GetSequencePose(OutAnimationPoseData, ExtractionContext, *this, RefFrameIndex, CanEvaluateRawAnimationData());
			break;
		}
		// use ref pose of Skeleton as base
		case ABPT_RefPose:
		default:
			OutAnimationPoseData.GetPose().ResetToRefPose();
			break;
	}
}

void UAnimSequence::GetBonePose_AdditiveMeshRotationOnly(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const
{
	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	if (!IsValidAdditive())
	{
		// since this is additive, need to initialize to identity
		OutPose.ResetToAdditiveIdentity();
		return;
	}

	// Get target pose
	GetBonePose(OutAnimationPoseData, ExtractionContext, CanEvaluateRawAnimationData());

	// get base pose
	FCompactPose BasePose;
	FBlendedCurve BaseCurve;
	UE::Anim::FStackAttributeContainer BaseAttributes;

	FAnimationPoseData BasePoseData(BasePose, BaseCurve, BaseAttributes);

	BasePose.SetBoneContainer(&OutPose.GetBoneContainer());
	BaseCurve.InitFrom(OutCurve);

	GetAdditiveBasePose(BasePoseData, ExtractionContext);

	// Convert them to mesh rotation.
	FAnimationRuntime::ConvertPoseToMeshRotation(OutPose);
	FAnimationRuntime::ConvertPoseToMeshRotation(BasePose);

	// Turn into Additive
	FAnimationRuntime::ConvertPoseToAdditive(OutPose, BasePose);
	OutCurve.ConvertToAdditive(BaseCurve);

	UE::Anim::Attributes::ConvertToAdditive(BaseAttributes, OutAttributes);
}

bool UAnimSequence::CanEvaluateRawAnimationData() const
{
#if WITH_EDITOR
	return IsDataModelValid();
#else
	return false;
#endif
}

const FCompressedAnimSequence& UAnimSequence::GetCompressedData_Internal() const
{
#if WITH_EDITOR
	if(!GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		static thread_local FCompressedAnimSequence DefaultCompressedAnimSequence;

		if (CurrentPlatformData)
		{
			return *CurrentPlatformData;
		}
		
		const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();

		if(DataByPlatformKeyHash.Num())
		{
			const FIoHash CurrentPlatformHash = GetDerivedDataKeyHash(RunningPlatform);
			if (const TUniquePtr<FCompressedAnimSequence>* PlatformCompressedData = DataByPlatformKeyHash.Find(CurrentPlatformHash))
			{
				return *PlatformCompressedData->Get();
			}
		}

		return DefaultCompressedAnimSequence;
	}
#endif // WITH_EDITOR

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CompressedData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


const FCompressedAnimSequence& UAnimSequence::GetPlatformCompressedData(const FAnimExtractContext& AnimExtractContext) const
{
#if WITH_EDITOR
	if(!GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		FIoHash Hash = FIoHash::Zero;	
		static thread_local FCompressedAnimSequence DefaultCompressedAnimSequence;	
		if (AnimExtractContext.bEnforceCompressedDataSampling)
		{
			if(AnimExtractContext.TargetPlatformHash.IsZero())
			{
				// Not set, check platform instead
				if (AnimExtractContext.TargetPlatform)
				{
					Hash = GetDerivedDataKeyHash(AnimExtractContext.TargetPlatform);
				}
				else
				{
					const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
					Hash = GetDerivedDataKeyHash(RunningPlatform);
				}
			}
			else
			{
				Hash = AnimExtractContext.TargetPlatformHash;
			}
		}
		else
		{
			if (CurrentPlatformData)
			{
				return *CurrentPlatformData;
			}
			
			const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
			Hash = GetDerivedDataKeyHash(RunningPlatform);
		}
	
		if (const TUniquePtr<FCompressedAnimSequence>* PlatformData = DataByPlatformKeyHash.Find(Hash))
		{
			return *PlatformData->Get();
		}

		return DefaultCompressedAnimSequence;
	}
#endif // WITH_EDITOR

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CompressedData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FCompressedAnimSequence& UAnimSequence::GetPlatformCompressedData(const FAnimExtractContext& AnimExtractContext)
{	
#if WITH_EDITOR
	if(!GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		FIoHash Hash = FIoHash::Zero;	
		static thread_local FCompressedAnimSequence DefaultCompressedAnimSequence;	
		if (AnimExtractContext.bEnforceCompressedDataSampling)
		{		
			if(AnimExtractContext.TargetPlatformHash.IsZero())
			{
				// Not set, check platform instead
				if (AnimExtractContext.TargetPlatform)
				{
					Hash = GetDerivedDataKeyHash(AnimExtractContext.TargetPlatform);
				}
				else
				{
					const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
					Hash = GetDerivedDataKeyHash(RunningPlatform);
				}
			}
			else
			{
				Hash = AnimExtractContext.TargetPlatformHash;
			}
		}
		else
		{
			if (CurrentPlatformData)
			{
				return *CurrentPlatformData;
			}
			
			const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
			Hash = GetDerivedDataKeyHash(RunningPlatform);
		}
		
		if (TUniquePtr<FCompressedAnimSequence>* PlatformData = DataByPlatformKeyHash.Find(Hash))
		{
			return *PlatformData->Get();
		}

		return DefaultCompressedAnimSequence;
	}
#endif // WITH_EDITOR

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CompressedData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


FCompressedAnimSequence& UAnimSequence::GetPlatformCompressedData(const ITargetPlatform* InTargetPlatform)
{	
#if WITH_EDITOR
	if(!GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		thread_local FCompressedAnimSequence DefaultCompressedAnimSequence;
		
		check(InTargetPlatform);
		
		const FIoHash Hash = GetDerivedDataKeyHash(InTargetPlatform);
		check(!Hash.IsZero());

		if (const TUniquePtr<FCompressedAnimSequence>* PlatformData = DataByPlatformKeyHash.Find(Hash))
		{
			return *PlatformData->Get();
		}

		return DefaultCompressedAnimSequence;
	}
#endif // WITH_EDITOR
	
	return GetCompressedData_Internal();
}

#if WITH_EDITOR
const FCompressedAnimSequence& UAnimSequence::GetPlatformCompressedData(const ITargetPlatform* InTargetPlatform) const
{
	if(!GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		thread_local FCompressedAnimSequence DefaultCompressedAnimSequence;
		
		check(InTargetPlatform);
		
		const FIoHash Hash = GetDerivedDataKeyHash(InTargetPlatform);
		check(!Hash.IsZero());

		if (const TUniquePtr<FCompressedAnimSequence>* PlatformData = DataByPlatformKeyHash.Find(Hash))
		{
			return *PlatformData->Get();
		}

		return DefaultCompressedAnimSequence;
	}

	return GetCompressedData_Internal();
}
#endif // WITH_EDITOR


#if WITH_EDITORONLY_DATA
void UAnimSequence::UpdateCompressedCurveName(const FName& OldCurveName, const FName& NewCurveName)
{
	UE_COMPRESSED_DATA_WRITE_SCOPE(this);
	for (TPair<FIoHash, TUniquePtr<FCompressedAnimSequence>>& CompressedDataPair : DataByPlatformKeyHash)
	{
		FCompressedAnimSequence& PlatformCompressedData = *CompressedDataPair.Value.Get();

		for (FAnimCompressedCurveIndexedName& IndexedCurveName : PlatformCompressedData.IndexedCurveNames)
		{
			if (IndexedCurveName.CurveName == OldCurveName)
			{
				IndexedCurveName.CurveName = NewCurveName;
				break;
			}
		}
		PlatformCompressedData.RebuildCurveIndexTable();		
	}
}
#endif // WITH_EDITORONLY_DATA

FTransform UAnimSequence::ExtractRootTrackTransform_Lockless(const FAnimExtractContext& ExtractionContext, const FBoneContainer* RequiredBones) const
{	
	const int32 RootBoneIndex = 0;

	bool bValidCompressedData = true;
	
	bool bContainsRootBoneTrack = [this, RequiredBones, RootBoneIndex, &ExtractionContext, &bValidCompressedData]()
	{
		const FCompressedAnimSequence& PlatformCompressedData = GetPlatformCompressedData(ExtractionContext);

		bool bUseRawDataForPoseExtraction = false;
#if WITH_EDITOR
		bValidCompressedData = PlatformCompressedData.IsValid(this);
		if (RequiredBones)
		{
			bUseRawDataForPoseExtraction = RequiredBones->ShouldUseRawData();
		}
		else
		{
			// Allow reading raw data in editor, as long as we don't explcitly request compressed data
			bUseRawDataForPoseExtraction = !bValidCompressedData && !ExtractionContext.bEnforceCompressedDataSampling; 
		}

		check(!ExtractionContext.bEnforceCompressedDataSampling || (bValidCompressedData && !bUseRawDataForPoseExtraction));	
#endif // WITH_EDITOR
		
#if WITH_EDITOR
		if (bUseRawDataForPoseExtraction)
		{
			if(CanEvaluateRawAnimationData())
			{
				ValidateModel();

				const FName RootBoneName = GetSkeleton()->GetReferenceSkeleton().GetBoneName(RootBoneIndex);
				if (DataModelInterface->IsValidBoneTrackName(RootBoneName))
				{
					return true;
				}
			}
		}
		else
#endif // WITH_EDITOR
		{
			const TArray<FTrackToSkeletonMap>& TrackToSkeletonMap = PlatformCompressedData.CompressedTrackToSkeletonMapTable;
			// we assume root is in first data if available = SkeletonIndex == 0 && BoneTreeIndex == 0)
			if ((TrackToSkeletonMap.Num() > 0) && (TrackToSkeletonMap[0].BoneTreeIndex == RootBoneIndex))
			{
				return true;
			}
		}

		return false;
	}();

	if(bContainsRootBoneTrack)
	{
		// if we do have root data, then return root data
		FTransform RootTransform;
		GetBoneTransform_Lockless(RootTransform, FSkeletonPoseBoneIndex(RootBoneIndex), ExtractionContext, !bValidCompressedData);
		return RootTransform;
	}

	// Fallback to root bone from reference skeleton.
	if( RequiredBones )
	{
		const FReferenceSkeleton& RefSkeleton = RequiredBones->GetReferenceSkeleton();
		if( RefSkeleton.GetNum() > 0 )
		{
			return RefSkeleton.GetRefBonePose()[RootBoneIndex];
		}
	}

	USkeleton * MySkeleton = GetSkeleton();
	// If we don't have a RequiredBones array, get root bone from default skeleton.
	if( !RequiredBones &&  MySkeleton )
	{
		const FReferenceSkeleton& RefSkeleton = MySkeleton->GetReferenceSkeleton();
		if( RefSkeleton.GetNum() > 0 )
		{
			return RefSkeleton.GetRefBonePose()[RootBoneIndex];
		}
	}

	// Otherwise, use identity.
	return FTransform::Identity;
}

FCompressedAnimSequence& UAnimSequence::GetCompressedData_Internal()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CompressedData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
void UAnimSequence::SetRetargetSourceAsset(USkeletalMesh* InRetargetSourceAsset)
{
	if (InRetargetSourceAsset != nullptr && InRetargetSourceAsset->HasAnyFlags(RF_Transient))
	{
		UE_LOGF(LogAnimation, Error, "Error, Transient asset [%ls] can not be assigned as Retarget Source for AnimSequence [%ls]. Please, use a non transient asset as retarget surce."
			, *(InRetargetSourceAsset->GetFullName())
			, *GetFullName());
		ensure(false);
		return;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RetargetSourceAsset = InRetargetSourceAsset;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const TSoftObjectPtr<USkeletalMesh>& UAnimSequence::GetRetargetSourceAsset() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RetargetSourceAsset;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAnimSequence::ClearRetargetSourceAsset()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RetargetSourceAsset.Reset();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAnimSequence::UpdateRetargetSourceAssetData()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	USkeletalMesh* SourceReferenceMesh = RetargetSourceAsset.LoadSynchronous();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const USkeleton* MySkeleton = GetSkeleton();
	if (SourceReferenceMesh && MySkeleton)
	{
		FAnimationRuntime::MakeSkeletonRefPoseFromMesh(SourceReferenceMesh, MySkeleton, RetargetSourceAssetReferencePose);
	}
	else
	{
		RetargetSourceAssetReferencePose.Empty();
	}
}
#endif // WITH_EDITOR

const TArray<FTransform>& UAnimSequence::GetRetargetTransforms() const
{
	if (RetargetSource.IsNone() && RetargetSourceAssetReferencePose.Num() > 0)
	{
		return RetargetSourceAssetReferencePose;
	}
	else
	{
		const USkeleton* MySkeleton = GetSkeleton();
		if (MySkeleton)
		{
			return MySkeleton->GetRefLocalPoses(RetargetSource);
		}
		else
		{
			static TArray<FTransform> EmptyTransformArray;
			return EmptyTransformArray;
		}
	}
}

FName UAnimSequence::GetRetargetTransformsSourceName() const
{
	if (RetargetSource.IsNone() && RetargetSourceAssetReferencePose.Num() > 0)
	{
		return GetOutermost()->GetFName();
	}
	else
	{
		return RetargetSource;
	}
}

void UAnimSequence::RetargetBoneTransform(FTransform& BoneTransform, const int32 SkeletonBoneIndex, const FCompactPoseBoneIndex& BoneIndex, const FBoneContainer& RequiredBones, const bool bIsBakedAdditive) const
{
	FAnimationRuntime::RetargetBoneTransform(GetSkeleton(), GetRetargetTransformsSourceName(), GetRetargetTransforms(), BoneTransform, SkeletonBoneIndex, BoneIndex, RequiredBones, bIsBakedAdditive);
}

#if WITH_EDITOR
bool UAnimSequence::ShouldPerformStripping(const bool bPerformFrameStripping, const bool bPerformStrippingOnOddFramedAnims) const
{
	const bool bShouldPerformFrameStripping = bPerformFrameStripping && bAllowFrameStripping;

	// Can only do stripping on animations that have an even number of frames once the end frame is removed)
	const bool bIsEvenFramed = ((NumberOfSampledKeys - 1) % 2) == 0;
	const bool bIsValidForStripping = bIsEvenFramed || bPerformStrippingOnOddFramedAnims;

	const bool bStripCandidate = (NumberOfSampledKeys > 10) && bIsValidForStripping;

	return bStripCandidate && bShouldPerformFrameStripping;
}

void UAnimSequence::WaitOnExistingCompression(const bool bWantResults)
{
	check(IsInGameThread());

	if(!bWantResults)
	{
		TryCancelAsyncTasks();
	}
	UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation({this});
}
#endif

void UAnimSequence::SerializeCompressedData(FArchive& Ar, bool bDDCData)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/SequenceData")); 
	
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		const ITargetPlatform* Platform = Ar.IsSaving() && Ar.IsCooking() ? Ar.CookingTarget() : GetTargetPlatformManagerRef().GetRunningTargetPlatform();
#else
		const ITargetPlatform* Platform = nullptr;
#endif // WITH_EDITOR
		
		if (Ar.IsLoading())
		{
			UE_COMPRESSED_DATA_WRITE_SCOPE(this);
			FCompressedAnimSequence& CompressedDataToSerialize = GetPlatformCompressedData(Platform);
			CompressedDataToSerialize.SerializeCompressedData(Ar, bDDCData, this, this->GetSkeleton(), BoneCompressionSettings, CurveCompressionSettings);
		}
		else
		{
			UE_COMPRESSED_DATA_READ_SCOPE(this);
			FCompressedAnimSequence& CompressedDataToSerialize = GetPlatformCompressedData(Platform);
			check(!Ar.IsCooking() || CompressedDataToSerialize.IsValid(this, true));
			CompressedDataToSerialize.SerializeCompressedData(Ar, bDDCData, this, this->GetSkeleton(), BoneCompressionSettings, CurveCompressionSettings);
		}
	}
}

void UAnimSequence::SerializeCompressedData(FArchive& Ar, FCompressedAnimSequence& CompressedDataToSerialize)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/SequenceData"));

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		check(!Ar.IsCooking() || CompressedDataToSerialize.IsValid(this, true));
		CompressedDataToSerialize.SerializeCompressedData(Ar, false, this, this->GetSkeleton(), BoneCompressionSettings, CurveCompressionSettings);
	}
}

#if WITH_EDITOR
void UAnimSequence::OnAnimModelLoaded()
{
	Super::OnAnimModelLoaded();

	if (ShouldDataModelBeValid())
	{
		CalculateNumberOfSampledKeys();
		Controller->RemoveBoneTracksMissingFromSkeleton(GetSkeleton());

		if (FAnimationUtils::GetForcedRecompressionSetting())
		{
			BeginCacheDerivedDataForCurrentPlatform();
		}

		if ( DataModelInterface->GetNumberOfKeys() == 0 && DataModelInterface->GetNumberOfFloatCurves() == 0 )
		{
			UE_LOGF(LogAnimation, Warning, "No animation data exists for sequence %ls (%ls)", *GetName(), (GetOuter() ? *GetOuter()->GetFullName() : *GetFullName()) );

			if (!IsRunningGame())
			{
				static FName NAME_LoadErrors("LoadErrors");
				FMessageLog LoadErrors(NAME_LoadErrors);

				TSharedRef<FTokenizedMessage> Message = LoadErrors.Warning();
				Message->AddToken(FTextToken::Create(LOCTEXT("EmptyAnimationData1", "The Animation ")));
				Message->AddToken(FAssetNameToken::Create(GetPathName(), FText::FromString(GetName())));
				Message->AddToken(FTextToken::Create(LOCTEXT("EmptyAnimationData2", " has no animation data. Recommend to remove.")));
				LoadErrors.Notify();
			}
		}

		if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MoveCustomAttributesToDataModel)
		{
			ValidateModel();
			MoveAttributesToModel();
		}

		// Do not try to cache compressed animation data when cooking, as cook target platform might not match current platform
		if (!IsRunningCookCommandlet())
		{
			BeginCacheDerivedDataForCurrentPlatform();			
		}
	}
}

bool UAnimSequence::CanBakeAdditive() const
{
	return	(NumberOfSampledKeys > 0) &&
			IsValidAdditive() &&
			GetSkeleton();
}

bool UAnimSequence::DoesSequenceContainZeroScale() const
{
	ValidateModel();

	TArray<FName> BoneNames;
	DataModelInterface->GetBoneTrackNames(BoneNames);
	for (const FName& BoneName : BoneNames)
	{
		bool bZeroScale = false;
		DataModelInterface->IterateBoneKeys(BoneName, [&bZeroScale](const FVector3f& Position, const FQuat4f& Rotation, const FVector3f& Scale, const FFrameNumber& FrameNumber)
		{
			bZeroScale = Scale.IsZero();
			return !bZeroScale;
		});

		if (bZeroScale)
		{
			return true;
		}
	}

	return false;
}

void UAnimSequence::FlagDependentAnimationsAsRawDataOnly() const
{
	for (TObjectIterator<UAnimSequence> Iter; Iter; ++Iter)
	{
		UAnimSequence* Seq = *Iter;
		/** Ignore self, as data will already be recompressed at higher-level callsite */
		if (Seq->HasAllFlags(RF_LoadCompleted) && Seq != this && Seq->RefPoseSeq == this)
		{
			Seq->ClearAllCompressionData();
			Seq->BeginCacheDerivedDataForCurrentPlatform();
		}
	}
}

void UAnimSequence::UpdateDependentStreamingAnimations() const
{
	for (TObjectIterator<UAnimStreamable> Iter; Iter; ++Iter)
	{
		UAnimStreamable* Seq = *Iter;
		if (Seq->SourceSequence == this)
		{
			Seq->InitFrom(this);
		}
	}
}

#endif

#if WITH_EDITOR
void UAnimSequence::ResetAnimation()
{
	// clear everything. Making new animation, so need to reset all the things that belong here
	ValidateModel();

	PlatformTargetFrameRate = UAnimationSettings::Get()->GetDefaultFrameRate();

	Controller->OpenBracket(LOCTEXT("ResetAnimation_Bracket", "Resetting Animation Sequence"));
	{
		Controller->ResetModel();
		DeleteNotifyTrackData();
	}
	Controller->CloseBracket();

	Notifies.Empty();
	AuthoredSyncMarkers.Empty();
	UniqueMarkerNames.Empty();
	AnimNotifyTracks.Empty();
}

void UAnimSequence::DeleteDeprecatedRawAnimationData()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RawAnimationData.Empty();
	SourceRawAnimationData_DEPRECATED.Empty();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAnimSequence::DeleteNotifyTrackData()
{
	AuthoredSyncMarkers.Empty();
	UniqueMarkerNames.Empty();
	Notifies.Empty();
	AnimNotifyTracks.Empty();

	InitializeNotifyTrack();
}

void UAnimSequence::ResetCompressionSettings()
{	
	UE_COMPRESSED_DATA_WRITE_SCOPE(this);
	
	BoneCompressionSettings = nullptr;
	CurveCompressionSettings = nullptr;
	ValidateCompressionSettings();
}

void UAnimSequence::DeleteBoneAnimationData()
{
	ValidateModel();
	Controller->RemoveAllBoneTracks();
}
#endif // WITH_EDITOR

bool UAnimSequence::IsValidAdditive() const		
{ 
	if (AdditiveAnimType != AAT_None)
	{
		bool bIsAdditiveRefPoseStripped = UE::Anim::Private::ShouldStripAdditiveRefPose();
#if WITH_EDITOR
		const bool bIsCookedData = GetPackage()->HasAnyPackageFlags(PKG_Cooked);
		if (!bIsCookedData)
		{
			bIsAdditiveRefPoseStripped = false;
		}
#endif
		switch (RefPoseType)
		{
		case ABPT_RefPose:
			return true;
		case ABPT_AnimScaled:
			return bIsAdditiveRefPoseStripped || (RefPoseSeq != nullptr);
		case ABPT_AnimFrame:
			return (bIsAdditiveRefPoseStripped || RefPoseSeq != nullptr) && (RefFrameIndex >= 0);
		case ABPT_LocalAnimFrame:
			return (RefFrameIndex >= 0);
		default:
			return false;
		}
	}

	return false;
}

UAnimSequence::FScopedCompressedAnimSequence UAnimSequence::GetCompressedData(const FAnimExtractContext& AnimExtractContext) const
{
	const FCompressedAnimSequence& PlatformCompressedData = GetPlatformCompressedData(AnimExtractContext);
	return FScopedCompressedAnimSequence(this, &PlatformCompressedData);
}

#if WITH_EDITOR
int32 FindMeshBoneIndexFromBoneName(USkeleton * Skeleton, const FName &BoneName)
{
	USkeletalMesh * PreviewMesh = Skeleton->GetPreviewMesh();
	const int32& SkeletonBoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);

	int32 BoneIndex = INDEX_NONE;

	if(SkeletonBoneIndex != INDEX_NONE)
	{
		BoneIndex = Skeleton->GetMeshBoneIndexFromSkeletonBoneIndex(PreviewMesh, SkeletonBoneIndex);
	}

	return BoneIndex;
}

void UAnimSequence::RemapTracksToNewSkeleton( USkeleton* NewSkeleton, bool bConvertSpaces )
{
	// this is not cheap, so make sure it only happens in editor

	ValidateModel();

	IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("RemapTrackToNewSkeleton_Bracket", "Remapping Bone Animation tracks to new Skeleton"));
	Controller->UpdateAttributesFromSkeleton(NewSkeleton);

	// @Todo : currently additive will work fine since we don't bake anything except when we extract
	// but in the future if we bake this can be problem
	if (bConvertSpaces)
	{
		USkeleton* OldSkeleton = GetSkeleton();
		
		// this will try to do bone to bone mapping
		if(OldSkeleton && OldSkeleton != NewSkeleton)
		{
			// Validate animation tracks against the new skeleton, any tracks linked to bones that do not exist in the new hierarchy are removed
			Controller->RemoveBoneTracksMissingFromSkeleton(NewSkeleton);
			ensure(DataModelInterface->GetNumBoneTracks() != 0);

			// make sure you do update reference pose before coming here
			
			// first calculate component space ref pose to get the relative transform between
			// two ref poses. It is very important update ref pose before getting here. 
			TArray<FTransform> NewSpaceBaseRefPose, OldSpaceBaseRefPose, RelativeToNewTransform;
			// get the spacebases transform
			FAnimationRuntime::FillUpComponentSpaceTransformsRefPose(NewSkeleton, NewSpaceBaseRefPose);
			FAnimationRuntime::FillUpComponentSpaceTransformsRefPose(OldSkeleton, OldSpaceBaseRefPose);

			const TArray<FTransform>& OldRefPose = OldSkeleton->GetReferenceSkeleton().GetRefBonePose();
			const TArray<FTransform>& NewRefPose = NewSkeleton->GetReferenceSkeleton().GetRefBonePose();

			// now we'd like to get the relative transform from old to new ref pose in component space
			// PK2*K2 = PK1*K1*theta where theta => P1*R1*theta = P2*R2 
			// where	P1 - parent transform in component space for original skeleton
			//			R1 - local space of the current bone for original skeleton
			//			P2 - parent transform in component space for new skeleton
			//			R2 - local space of the current bone for new skeleton
			// what we're looking for is theta, so that we can apply that to animated transform
			int32 NumBones = NewSpaceBaseRefPose.Num();
			// saves the theta data per bone
			RelativeToNewTransform.AddUninitialized(NumBones);
			TArray<float> OldToNewTranslationRatio;
			// saves the translation conversion data
			OldToNewTranslationRatio.AddUninitialized(NumBones);

			// calculate the relative transform to new skeleton
			// so that we can apply the delta in component space
			for(int32 BoneIndex=0; BoneIndex<NumBones; ++BoneIndex)
			{
				// first find bone name of the idnex
				FName BoneName = NewSkeleton->GetReferenceSkeleton().GetRefBoneInfo()[BoneIndex].Name;
				// find it in old index
				int32 OldBoneIndex = OldSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);

				// get old bone index
				if(OldBoneIndex != INDEX_NONE)
				{
					// theta (RelativeToNewTransform) = (P1*R1)^(-1) * P2*R2 where theta => P1*R1*theta = P2*R2
					RelativeToNewTransform[BoneIndex] = NewSpaceBaseRefPose[BoneIndex].GetRelativeTransform(OldSpaceBaseRefPose[OldBoneIndex]);

					// also savees the translation difference between old to new
					FVector OldTranslation = OldRefPose[OldBoneIndex].GetTranslation();
					FVector NewTranslation = NewRefPose[BoneIndex].GetTranslation();

					float OldTranslationSize = OldTranslation.Size();
					float NewTranslationSize = NewTranslation.Size();
					OldToNewTranslationRatio[BoneIndex] = (FMath::IsNearlyZero(OldTranslationSize))? 1.f/*do not touch new translation size*/ : NewTranslationSize/OldTranslationSize;
				}
				else
				{
					RelativeToNewTransform[BoneIndex].SetIdentity();
				}
			}

			// 2d array of animated time [boneindex][time key]
			TArray< TArray<FTransform> > AnimatedSpaceBases, ConvertedLocalSpaces, ConvertedSpaceBases;
			AnimatedSpaceBases.AddZeroed(NumBones);
			ConvertedLocalSpaces.AddZeroed(NumBones);
			ConvertedSpaceBases.AddZeroed(NumBones);

			const int32 NumKeys = DataModelInterface->GetNumberOfKeys();
			const float Interval = DataModelInterface->GetFrameRate().AsInterval();

			// allocate arrays
			for(int32 BoneIndex=0; BoneIndex<NumBones; ++BoneIndex)
			{
				AnimatedSpaceBases[BoneIndex].AddUninitialized(NumKeys);
				ConvertedLocalSpaces[BoneIndex].AddUninitialized(NumKeys);
				ConvertedSpaceBases[BoneIndex].AddUninitialized(NumKeys);
			}

			// now calculating old animated space bases
			// this one calculates aniamted space per bones and per key
			TArray<FTransform> BoneTransforms;
			for(int32 BoneIndex=0; BoneIndex<NumBones; ++BoneIndex)
			{
				FName BoneName = NewSkeleton->GetReferenceSkeleton().GetBoneName(BoneIndex);
				const int32 OldBoneIndex = OldSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
				const bool bContainsTrack = DataModelInterface->IsValidBoneTrackName(BoneName);
				const int32 ParentBoneIndex = NewSkeleton->GetReferenceSkeleton().GetParentIndex(BoneIndex);

				if(bContainsTrack)
				{
					// fill up keys - calculate PK1 * K1
					BoneTransforms.Reset();
					DataModelInterface->GetBoneTrackTransforms(BoneName, BoneTransforms);
					for(int32 Key=0; Key<NumKeys; ++Key)
					{
						FTransform AnimatedLocalKey = BoneTransforms[Key];
						// note that we apply scale in the animated space
						// at this point, you should have scaled version of animated skeleton
						AnimatedLocalKey.ScaleTranslation(OldToNewTranslationRatio[BoneIndex]);

						if(ParentBoneIndex != INDEX_NONE)
						{
							AnimatedSpaceBases[BoneIndex][Key] = AnimatedLocalKey * AnimatedSpaceBases[ParentBoneIndex][Key];
						}
						else
						{
							AnimatedSpaceBases[BoneIndex][Key] = AnimatedLocalKey;
						}
					}
				}
				else
				{
					// get local spaces from refpose and use that to fill it up
					FTransform LocalTransform = (OldBoneIndex != INDEX_NONE)? OldSkeleton->GetReferenceSkeleton().GetRefBonePose()[OldBoneIndex] : FTransform::Identity;

					for(int32 Key=0; Key<NumKeys; ++Key)
					{
						if(ParentBoneIndex != INDEX_NONE)
						{
							AnimatedSpaceBases[BoneIndex][Key] = LocalTransform * AnimatedSpaceBases[ParentBoneIndex][Key];
						}
						else
						{
							AnimatedSpaceBases[BoneIndex][Key] = LocalTransform;
						}
					}
				}
			}

			TArray<FVector3f> PosKeys;
			TArray<FQuat4f> RotKeys;
			TArray<FVector3f> ScaleKeys;

			// now apply the theta back to the animated space bases
			for(int32 BoneIndex=0; BoneIndex<NumBones; ++BoneIndex)
			{
				const FName BoneName = NewSkeleton->GetReferenceSkeleton().GetBoneName(BoneIndex);
				int32 ParentBoneIndex = NewSkeleton->GetReferenceSkeleton().GetParentIndex(BoneIndex);

				for(int32 Key=0; Key<NumKeys; ++Key)
				{
					// thus PK2 & K2 =  PK1 * K1 * theta where theta = (P1*R1)^(-1) * P2*R2
					// where PK2	: parent transform in component space of animated key for new skeleton
					//		 K2		: local transform of animated key for new skeleton
					//		 PK1	: parent transform in component space of animated key for old skeleton
					//		 K1		: local transform of animated key for old skeleton
					FTransform SpaceBase;
					// we don't just apply it because translation is sensitive
					// we don't like to apply relative transform to tranlsation directly
					// rotation and scale we can, but translation we'd like to use scaled translation instead of transformed location
					// as their relative translation can be different
					SpaceBase.SetRotation(AnimatedSpaceBases[BoneIndex][Key].GetRotation() * RelativeToNewTransform[BoneIndex].GetRotation());
					SpaceBase.SetScale3D(AnimatedSpaceBases[BoneIndex][Key].GetScale3D() * RelativeToNewTransform[BoneIndex].GetScale3D());
					// use animated scaled translation directly
					SpaceBase.SetTranslation(AnimatedSpaceBases[BoneIndex][Key].GetTranslation());
					ConvertedSpaceBases[BoneIndex][Key] = SpaceBase;
					// now calculate local space for animation
					if(ParentBoneIndex != INDEX_NONE)
					{
						// K2 = PK2^(-1) * PK1 * K1 * (P1*R1)^(-1) * P2*R2
						ConvertedLocalSpaces[BoneIndex][Key] = SpaceBase.GetRelativeTransform(ConvertedSpaceBases[ParentBoneIndex][Key]);
					}
					else
					{
						ConvertedLocalSpaces[BoneIndex][Key] = SpaceBase;
					}
				}

				// now save back to animation data
				if(DataModelInterface->IsValidBoneTrackName(BoneName))
				{
					PosKeys.SetNumUninitialized(NumKeys);
					RotKeys.SetNumUninitialized(NumKeys);
					ScaleKeys.SetNumUninitialized(NumKeys);

					for(int32 Key=0; Key<NumKeys; ++Key)
					{
						PosKeys[Key] = FVector3f(ConvertedLocalSpaces[BoneIndex][Key].GetLocation());
						RotKeys[Key] = FQuat4f(ConvertedLocalSpaces[BoneIndex][Key].GetRotation());
						ScaleKeys[Key] = FVector3f(ConvertedLocalSpaces[BoneIndex][Key].GetScale3D());
					}

					Controller->SetBoneTrackKeys(BoneName, PosKeys, RotKeys, ScaleKeys);
				}
			}
		}
		else
		{
			Controller->UpdateWithSkeleton(NewSkeleton);
		}

		// I have to set this here in order for compression
		// that has to happen outside of this after Skeleton changes
		SetSkeleton(NewSkeleton);
	}
	else
	{
		if (NewSkeleton != GetSkeleton() || NewSkeleton->GetGuid() != GetSkeletonGuid())
		{
			SetSkeleton(NewSkeleton);
			Controller->UpdateWithSkeleton(NewSkeleton, false);
		}
	}

	Super::RemapTracksToNewSkeleton(NewSkeleton, bConvertSpaces);
}

int32 FindFirstChildTrack(const USkeleton* MySkeleton, const FReferenceSkeleton& RefSkeleton, const TArray<FName>& AnimationTrackNames, FName BoneName)
{
	const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
	if(BoneIndex == INDEX_NONE)
	{
		// get out, nothing to do
		return INDEX_NONE;
	}

	// find children
	TArray<int32> Childs;
	if(MySkeleton->GetChildBones(BoneIndex, Childs) > 0)
	{
		// first look for direct children
		for(auto ChildIndex : Childs)
		{
			FName ChildBoneName = RefSkeleton.GetBoneName(ChildIndex);
			int32 ChildTrackIndex = AnimationTrackNames.Find(ChildBoneName);
			if(ChildTrackIndex != INDEX_NONE)
			{
				// found the new track
				return ChildTrackIndex;
			}
		}

		int32 BestGrandChildIndex = INDEX_NONE;
		// if you didn't find yet, now you have to go through all children
		for(auto ChildIndex : Childs)
		{
			FName ChildBoneName = RefSkeleton.GetBoneName(ChildIndex);
			// now I have to go through all childrewn and find who is earliest since I don't know which one might be the closest one
			int32 GrandChildIndex = FindFirstChildTrack(MySkeleton, RefSkeleton, AnimationTrackNames, ChildBoneName);
			if (GrandChildIndex != INDEX_NONE)
			{
				if (BestGrandChildIndex == INDEX_NONE)
				{
					BestGrandChildIndex = GrandChildIndex;
				}
				else if (BestGrandChildIndex > GrandChildIndex)
				{
					// best should be earlier track index
					BestGrandChildIndex = GrandChildIndex;
				}
			}
		}

		return BestGrandChildIndex;
	}
	else
	{
		// there is no child, just add at the end
		return AnimationTrackNames.Num();
	}
}

bool UAnimSequence::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive /*= true*/)
{
	Super::GetAllAnimationSequencesReferred(AnimationAssets, bRecursive);
	if (RefPoseSeq  && RefPoseSeq != this && !AnimationAssets.Contains(RefPoseSeq))
	{
		RefPoseSeq->HandleAnimReferenceCollection(AnimationAssets, bRecursive);
	}
	return AnimationAssets.Num() > 0;
}

#if WITH_EDITOR
UAnimSequence::FScopedCompressedAnimSequence UAnimSequence::GetCompressedData(const ITargetPlatform* InTargetPlatform) const
{
	const FCompressedAnimSequence& PlatformCompressedData = GetPlatformCompressedData(InTargetPlatform);
	return FScopedCompressedAnimSequence(this, &PlatformCompressedData);
}
#endif // WITH_EDITOR

void UAnimSequence::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	Super::ReplaceReferredAnimations(ReplacementMap);

	if (RefPoseSeq)
	{
		UAnimSequence* const* ReplacementAsset = (UAnimSequence*const*)ReplacementMap.Find(RefPoseSeq);
		if (ReplacementAsset)
		{
			RefPoseSeq = *ReplacementAsset;
		}
	}
}

int32 UAnimSequence::GetSpaceBasedAnimationData(TArray< TArray<FTransform> >& AnimationDataInComponentSpace) const
{
	USkeleton* MySkeleton = GetSkeleton();

	check(MySkeleton);
	const FReferenceSkeleton& RefSkeleton = MySkeleton->GetReferenceSkeleton();
	int32 NumBones = RefSkeleton.GetNum();

	AnimationDataInComponentSpace.Empty(NumBones);
	AnimationDataInComponentSpace.AddZeroed(NumBones);

	ValidateModel();

	// 2d array of animated time [boneindex][time key]
	const int32 NumKeys = DataModelInterface->GetNumberOfKeys();
	const float Interval = DataModelInterface->GetFrameRate().AsInterval();

	// allocate arrays
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		AnimationDataInComponentSpace[BoneIndex].AddUninitialized(NumKeys);
	}
	
	// now calculating old animated space bases
	// this one calculates animated space per bones and per key
	TArray<FTransform> BoneTransforms;
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const FName BoneName = MySkeleton->GetReferenceSkeleton().GetBoneName(BoneIndex);
		const int32 ParentBoneIndex = MySkeleton->GetReferenceSkeleton().GetParentIndex(BoneIndex);

		if (DataModelInterface->IsValidBoneTrackName(BoneName))
		{
			BoneTransforms.Reset();
			DataModelInterface->GetBoneTrackTransforms(BoneName, BoneTransforms);
			// fill up keys - calculate PK1 * K1
			for (int32 Key = 0; Key < NumKeys; ++Key)
			{
				FTransform AnimatedLocalKey = BoneTransforms[Key];
				if (ParentBoneIndex != INDEX_NONE)
				{
					AnimationDataInComponentSpace[BoneIndex][Key] = AnimatedLocalKey * AnimationDataInComponentSpace[ParentBoneIndex][Key];
				}
				else
				{
					AnimationDataInComponentSpace[BoneIndex][Key] = AnimatedLocalKey;
				}
			}
		}
		else
		{
			// get local spaces from refpose and use that to fill it up
			FTransform LocalTransform = MySkeleton->GetReferenceSkeleton().GetRefBonePose()[BoneIndex];

			for (int32 Key = 0; Key < NumKeys; ++Key)
			{
				if (ParentBoneIndex != INDEX_NONE)
				{
					AnimationDataInComponentSpace[BoneIndex][Key] = LocalTransform * AnimationDataInComponentSpace[ParentBoneIndex][Key];
				}
				else
				{
					AnimationDataInComponentSpace[BoneIndex][Key] = LocalTransform;
				}
			}
		}	
	}

	return AnimationDataInComponentSpace.Num();
}

void UAnimSequence::AddKeyToSequence(float Time, const FName& BoneName, const FTransform& AdditiveTransform)
{
	// find if this already exists, then just add curve data only
	FName CurveName = BoneName;
	USkeleton * CurrentSkeleton = GetSkeleton();
	check (CurrentSkeleton);
	
	ValidateModel();

	IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("AddKeyToSequence_Bracket", "Adding key to sequence"));
	FAnimationCurveIdentifier TransformCurveId(CurveName, ERawCurveTrackTypes::RCT_Transform);
	Controller->AddCurve(TransformCurveId, AACF_DriveTrack | AACF_Editable);

	const FTransformCurve* TransformCurve = DataModelInterface->FindTransformCurve(TransformCurveId);
	check(TransformCurve);

	Controller->SetTransformCurveKey(TransformCurveId, Time, AdditiveTransform);
}

uint8* UAnimSequence::FindSyncMarkerPropertyData(int32 SyncMarkerIndex, FArrayProperty*& ArrayProperty)
{
	ArrayProperty = nullptr;

	if (AuthoredSyncMarkers.IsValidIndex(SyncMarkerIndex))
	{
		return FindArrayProperty(TEXT("AuthoredSyncMarkers"), ArrayProperty, SyncMarkerIndex);
	}
	return nullptr;
}

bool UAnimSequence::IsCompressedDataOutOfDate() const
{
	return GetSkeleton() &&
	(
		!IsCompressedDataValid() ||
		GetSkeletonVirtualBoneGuid() != GetSkeleton()->GetVirtualBoneGuid()
	);
}

bool UAnimSequence::CreateAnimation(USkeletalMesh* Mesh)
{
	// create animation from Mesh's ref pose
	if (Mesh)
	{
		ValidateModel();
		const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();

		IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("CreateAnimationSkeletalMesh_Bracket", "Creating Animation Sequence based up Skeletal Mesh"));
		ResetAnimation();

		Controller->SetFrameRate(UAnimationSettings::Get()->GetDefaultFrameRate());
		Controller->SetNumberOfFrames(1);

		const int32 NumBones = RefSkeleton.GetRawBoneNum();

		const TArray<FTransform>& RefBonePose = RefSkeleton.GetRawRefBonePose();

		check(RefBonePose.Num() == NumBones);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const FName& BoneName = RefSkeleton.GetBoneName(BoneIndex);
			Controller->AddBoneCurve(BoneName);
			Controller->SetBoneTrackKeys(BoneName, { FVector3f(RefBonePose[BoneIndex].GetTranslation()), FVector3f(RefBonePose[BoneIndex].GetTranslation()) }, { FQuat4f(RefBonePose[BoneIndex].GetRotation()), FQuat4f(RefBonePose[BoneIndex].GetRotation()) }, { FVector3f(RefBonePose[BoneIndex].GetScale3D()), FVector3f(RefBonePose[BoneIndex].GetScale3D()) });
		}

		Controller->NotifyPopulated();
		

		return true;
	}

	return false;
}

bool UAnimSequence::CreateAnimation(USkeletalMeshComponent* MeshComponent)
{
	if(MeshComponent && MeshComponent->GetSkeletalMeshAsset())
	{
		USkeletalMesh * Mesh = MeshComponent->GetSkeletalMeshAsset();

		ValidateModel();
		const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();

		IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("CreateAnimationSkeletalMeshComponent_Bracket", "Creating Animation Sequence based up Skeletal Mesh Component"));
		ResetAnimation();

		Controller->SetFrameRate(UAnimationSettings::Get()->GetDefaultFrameRate());
		Controller->SetNumberOfFrames(1);

		const int32 NumBones = RefSkeleton.GetRawBoneNum();

		const TArray<FTransform> BoneSpaceTransforms = MeshComponent->GetBoneSpaceTransforms();

		check(BoneSpaceTransforms.Num() >= NumBones);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const FName& BoneName = RefSkeleton.GetBoneName(BoneIndex);
			Controller->AddBoneCurve(BoneName);
			Controller->SetBoneTrackKeys(BoneName, TArray<FVector>{ BoneSpaceTransforms[BoneIndex].GetTranslation(), BoneSpaceTransforms[BoneIndex].GetTranslation() }, { BoneSpaceTransforms[BoneIndex].GetRotation(), BoneSpaceTransforms[BoneIndex].GetRotation() }, TArray<FVector>{ BoneSpaceTransforms[BoneIndex].GetScale3D(), BoneSpaceTransforms[BoneIndex].GetScale3D() });
		}

		Controller->NotifyPopulated();
		return true;
	}

	return false;
}

bool UAnimSequence::CreateAnimation(UAnimSequence* Sequence)
{
	if(Sequence)
	{
		ResetAnimation();

		CopyDataModel(Sequence->GetDataModelInterface());
		Controller->SetModel(DataModelInterface);
		Controller->NotifyPopulated();

		UE::Anim::CopyNotifies(Sequence, this, false);

		return true;
	}
	
	return false;
}

#endif

void UAnimSequence::RefreshCacheData()
{
	SortSyncMarkers();
#if WITH_EDITOR
	for (int32 TrackIndex = 0; TrackIndex < AnimNotifyTracks.Num(); ++TrackIndex)
	{
		AnimNotifyTracks[TrackIndex].SyncMarkers.Empty();
	}
	for (FAnimSyncMarker& SyncMarker : AuthoredSyncMarkers)
	{
		const int32 TrackIndex = SyncMarker.TrackIndex;
		if (AnimNotifyTracks.IsValidIndex(TrackIndex))
		{
			AnimNotifyTracks[TrackIndex].SyncMarkers.Add(&SyncMarker);
		}
		else
		{
			// This should not happen, but if it does we must find somewhere else to add it
			ensureMsgf(0, TEXT("AnimNotifyTrack: Wrong indices found"));
			AnimNotifyTracks[0].SyncMarkers.Add(&SyncMarker);
			SyncMarker.TrackIndex = 0;
		}
	}
#endif
	Super::RefreshCacheData();
}

int32 UAnimSequence::GetNumberOfSampledKeys() const 
{ 
#if WITH_EDITOR
	return NumberOfSampledKeys;
#else
	return GetCompressedData_Internal().CompressedDataStructure ? GetCompressedData_Internal().CompressedDataStructure->CompressedNumberOfKeys : PlatformTargetFrameRate.Default.AsFrameTime(GetPlayLength()).RoundToFrame().Value + 1;
#endif
}

void UAnimSequence::EvaluateCurveData(FBlendedCurve& OutCurve, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData) const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	EvaluateCurveData_Lockless(OutCurve, AnimExtractContext, bForceUseRawData);
}

void UAnimSequence::EvaluateCurveData_Lockless(FBlendedCurve& OutCurve, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData) const
{
	SCOPE_CYCLE_COUNTER(STAT_AnimSeq_EvalCurveData);
	
	const FCompressedAnimSequence& PlatformCompressedData = GetPlatformCompressedData(AnimExtractContext);
	const bool bEvaluateRawData = bForceUseRawData || !PlatformCompressedData.IsCurveDataValid(this);
	check(!bForceUseRawData || CanEvaluateRawAnimationData());
	if (CanEvaluateRawAnimationData() && bEvaluateRawData)
	{
#if WITH_EDITOR
		UE::Anim::EvaluateFloatCurvesFromModel(DataModelInterface.GetInterface(), OutCurve, AnimExtractContext.CurrentTime);
#else
		Super::EvaluateCurveData(OutCurve, AnimExtractContext, bForceUseRawData);
#endif
	}
	else if(PlatformCompressedData.IsCurveDataValid(this))
	{
		if (PlatformCompressedData.CurveCompressionCodec)
		{
			CSV_SCOPED_TIMING_STAT(Animation, EvaluateCurveData);
			PlatformCompressedData.CurveCompressionCodec->DecompressCurves(PlatformCompressedData, OutCurve, AnimExtractContext.CurrentTime);
		}
	}
}

float UAnimSequence::EvaluateCurveData(FName CurveName, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData) const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	return EvaluateCurveData_Lockless(CurveName, AnimExtractContext, bForceUseRawData);
}

float UAnimSequence::EvaluateCurveData_Lockless(FName CurveName, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData) const
{
	QUICK_SCOPE_CYCLE_COUNTER(EvaluateCurveDataByName);

	const FCompressedAnimSequence& PlatformCompressedData = GetPlatformCompressedData(AnimExtractContext);
	const bool bEvaluateRawData = bForceUseRawData || !PlatformCompressedData.IsCurveDataValid(this);
	check(!bForceUseRawData || CanEvaluateRawAnimationData());
	if (CanEvaluateRawAnimationData() && bEvaluateRawData)
	{
		return Super::EvaluateCurveData(CurveName, AnimExtractContext, bForceUseRawData);
	}

	if(PlatformCompressedData.IsCurveDataValid(this))
	{
		if(PlatformCompressedData.CurveCompressionCodec)
		{
			return PlatformCompressedData.CurveCompressionCodec->DecompressCurve(PlatformCompressedData, CurveName, AnimExtractContext.CurrentTime);
		}
	}

	return 0.f;
}

bool UAnimSequence::HasCurveData(FName CurveName, bool bForceUseRawData) const
{
	const bool bEvaluateRawData = bForceUseRawData || !IsCurveCompressedDataValid();
	check(!bForceUseRawData || CanEvaluateRawAnimationData());
	if (CanEvaluateRawAnimationData() && bEvaluateRawData)
	{
		return Super::HasCurveData(CurveName, bForceUseRawData);
	}

	if(IsCurveCompressedDataValid())
	{
		UE_COMPRESSED_DATA_READ_SCOPE(this);
		const FCompressedAnimSequence& PlatformCompressedData = GetCompressedData_Internal();
		if(PlatformCompressedData.CurveCompressionCodec)
		{
			for (const FAnimCompressedCurveIndexedName& IndexedCurveName : PlatformCompressedData.IndexedCurveNames)
			{
				if (IndexedCurveName.CurveName == CurveName)
				{
					return true;
				}
			}
		}
	}

	return false;
}

void UAnimSequence::RefreshSyncMarkerDataFromAuthored()
{
#if WITH_EDITOR
	MarkerDataUpdateCounter++;
#endif

	if (AuthoredSyncMarkers.Num() > 0)
	{
		UniqueMarkerNames.Reset();
		UniqueMarkerNames.Reserve(AuthoredSyncMarkers.Num());

		const FAnimSyncMarker* PreviousMarker = nullptr;
		for (const FAnimSyncMarker& Marker : AuthoredSyncMarkers)
		{
			UniqueMarkerNames.AddUnique(Marker.MarkerName);
			PreviousMarker = &Marker;
		}
	}
	else
	{
		UniqueMarkerNames.Empty();
	}

#if WITH_EDITOR
	check(IsInGameThread());

	// Update blend spaces that may be referencing us
	TArray<UAnimationAsset*> ReferredAssets;
	for(TObjectIterator<UBlendSpace> It; It; ++It)
	{
		if(!It->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad))
		{
			ReferredAssets.Reset();
			It->GetAllAnimationSequencesReferred(ReferredAssets, false);
			if (ReferredAssets.Contains(this))
			{
				It->RuntimeValidateMarkerData();
			}
		}
	}
#endif
}

bool IsMarkerValid(const FAnimSyncMarker* Marker, bool bLooping, const TArray<FName>& ValidMarkerNames)
{
	return (Marker == nullptr && !bLooping) || (Marker && ValidMarkerNames.Contains(Marker->MarkerName));
}

void UAnimSequence::AdvanceMarkerPhaseAsLeader(bool bLooping, float MoveDelta, const TArray<FName>& ValidMarkerNames, float& CurrentTime, FMarkerPair& PrevMarker, FMarkerPair& NextMarker, TArray<FPassedMarker>& MarkersPassed, const UMirrorDataTable* MirrorTable) const
{
	check(MoveDelta != 0.f);
	const bool bPlayingForwards = MoveDelta >= 0.f;
	float CurrentMoveDelta = MoveDelta;

	// Hard to reproduce issue triggering this, ensure & clamp for now
	ensureMsgf(CurrentTime >= 0.f && CurrentTime <= GetPlayLength(), TEXT("Current time inside of AdvanceMarkerPhaseAsLeader is out of range %.3f of 0.0 to %.3f\n    Sequence: %s"), CurrentTime, GetPlayLength(), *GetFullName());

	// Ensure our time is within the boundaries of the anim sequence.
	CurrentTime = FMath::Clamp(CurrentTime, 0.f, GetPlayLength());

	if (bPlayingForwards)
	{
		// Repeat until there is no more move delta to handle.
		while (true)
		{
			// Our next marker is the end boundary. (Only possible if sequence is not looping)
			if (NextMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
			{
				const float PrevCurrentTime = CurrentTime;

				// Ensure we dont overshoot when advancing our time.
				CurrentTime = FMath::Min(CurrentTime + CurrentMoveDelta, GetPlayLength());

				// Compute the distances left to reach the next and previous marker from the current time position.
				NextMarker.TimeToMarker = GetPlayLength() - CurrentTime;
				PrevMarker.TimeToMarker -= CurrentTime - PrevCurrentTime;
				break;
			}

			// Good, we have a valid next marker.
			const FAnimSyncMarker& NextSyncMarker = AuthoredSyncMarkers[NextMarker.MarkerIndex];
			checkSlow(ValidMarkerNames.Contains(NextSyncMarker.MarkerName));

			// We are going to end up past our next marker.
			if (CurrentMoveDelta > NextMarker.TimeToMarker)
			{
				// Move time to match that of the next marker, and update the move delta to reflect the change.
				CurrentTime = NextSyncMarker.Time;
				CurrentMoveDelta -= NextMarker.TimeToMarker;

				// Make our new previous marker be the marker we just passed.
				PrevMarker.MarkerIndex = NextMarker.MarkerIndex; 
				PrevMarker.TimeToMarker = 0.0f;

				// Record that we just passed a marker.
				const int32 PassedMarker = MarkersPassed.Add(FPassedMarker());
				MarkersPassed[PassedMarker].PassedMarkerName = MarkerOrMirroredName(NextSyncMarker.MarkerName, MirrorTable);
				MarkersPassed[PassedMarker].DeltaTimeWhenPassed = CurrentMoveDelta;
				
				// Compute our new next marker.
				{
					float MarkerTimeOffset = 0.f;
					
					do
					{
						++NextMarker.MarkerIndex;

						// No more markers up ahead.
						if (NextMarker.MarkerIndex >= AuthoredSyncMarkers.Num())
						{
							// Stop at anim end boundary.
							if (!bLooping)
							{
								NextMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
								break;
							}

							// Make our next marker be the first marker found in the sequence. 
							NextMarker.MarkerIndex = 0;
							MarkerTimeOffset = GetPlayLength();
						}
					} while (!ValidMarkerNames.Contains(AuthoredSyncMarkers[NextMarker.MarkerIndex].MarkerName));

					// Update time left to reach the new next marker
					if (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary)
					{
						NextMarker.TimeToMarker = MarkerTimeOffset + AuthoredSyncMarkers[NextMarker.MarkerIndex].Time - CurrentTime;
					}
				}
			}
			// We will not go past our next marker, we can advance comfortably.
			else
			{
				CurrentTime = FMath::Fmod(CurrentTime + CurrentMoveDelta, GetPlayLength());
				if (CurrentTime < 0.f)
				{
					CurrentTime += GetPlayLength();
				}

				NextMarker.TimeToMarker -= CurrentMoveDelta;
				PrevMarker.TimeToMarker -= CurrentMoveDelta;
				break;
			}
		}
	}
	else
	{
		// Playing backwards.
		
		// Repeat until there is no more move delta to handle.
		while (true)
		{
			// Our previous marker is the start boundary.
			if (PrevMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
			{
				const float PrevCurrentTime = CurrentTime;

				// Ensure we dont undershoot when advancing our time.
				CurrentTime = FMath::Max(CurrentTime + CurrentMoveDelta, 0.f);

				// Compute the distances left to reach the next and previous marker from the current time position.
				PrevMarker.TimeToMarker = CurrentTime;
				NextMarker.TimeToMarker -= CurrentTime - PrevCurrentTime;
				break;
			}

			// Good, we have a valid previous marker.
			const FAnimSyncMarker& PrevSyncMarker = AuthoredSyncMarkers[PrevMarker.MarkerIndex];
			checkSlow(ValidMarkerNames.Contains(PrevSyncMarker.MarkerName));

			// We are going to end up past our previous marker.
			if (CurrentMoveDelta < PrevMarker.TimeToMarker)
			{
				// Move time to match that of the previous marker, and update the move delta to reflect the change.
				CurrentTime = PrevSyncMarker.Time;
				CurrentMoveDelta -= PrevMarker.TimeToMarker;

				// Make our new next marker be the marker we just passed.
				NextMarker.MarkerIndex = PrevMarker.MarkerIndex;
				NextMarker.TimeToMarker = 0.0f;

				// Record that we just passed a marker.
				const int32 PassedMarker = MarkersPassed.Add(FPassedMarker());
				MarkersPassed[PassedMarker].PassedMarkerName = MarkerOrMirroredName(PrevSyncMarker.MarkerName, MirrorTable);
				MarkersPassed[PassedMarker].DeltaTimeWhenPassed = CurrentMoveDelta;

				// Compute our new previous marker.
				{
					float MarkerTimeOffset = 0.f;
					
					do
					{
						--PrevMarker.MarkerIndex;

						// No more markers behind.
						if (PrevMarker.MarkerIndex < 0)
						{
							// Stop at the anim start boundary.
							if (!bLooping)
							{
								PrevMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
								break;
							}
							
							// Make our previous marker be the last marker found in the sequence.
							PrevMarker.MarkerIndex = AuthoredSyncMarkers.Num() - 1;
							MarkerTimeOffset -= GetPlayLength();
						}
					} while (!ValidMarkerNames.Contains(AuthoredSyncMarkers[PrevMarker.MarkerIndex].MarkerName));

					// Update time left to reach marker.
					if (PrevMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary)
					{
						PrevMarker.TimeToMarker = MarkerTimeOffset + AuthoredSyncMarkers[PrevMarker.MarkerIndex].Time - CurrentTime;
					}
				}
			}
			// We will not go past our previous marker, we can advance comfortably.
			else
			{
				CurrentTime = FMath::Fmod(CurrentTime + CurrentMoveDelta, GetPlayLength());
				if (CurrentTime < 0.f)
				{
					CurrentTime += GetPlayLength();
				}

				PrevMarker.TimeToMarker -= CurrentMoveDelta;
				NextMarker.TimeToMarker -= CurrentMoveDelta;
				break;
			}
		}
	}

	check(CurrentTime >= 0.f && CurrentTime <= GetPlayLength());
}

void AdvanceMarkerForwards(int32& Marker, FName MarkerToFind, bool bLooping, const TArray<FAnimSyncMarker>& AuthoredSyncMarkers, const UMirrorDataTable* MirrorTable)
{
	int32 MaxIterations = AuthoredSyncMarkers.Num();

	// Get next available marker.
	while ((MarkerOrMirroredName(AuthoredSyncMarkers[Marker].MarkerName, MirrorTable) != MarkerToFind) && (--MaxIterations >= 0))
	{
		++Marker;
		if (Marker == AuthoredSyncMarkers.Num() && !bLooping)
		{
			break;
		}
		Marker %= AuthoredSyncMarkers.Num();
	}

	// In any invalid case, default to -1 aka an animation boundary.
	if (!AuthoredSyncMarkers.IsValidIndex(Marker) || (MarkerOrMirroredName(AuthoredSyncMarkers[Marker].MarkerName, MirrorTable) != MarkerToFind))
	{
		Marker = MarkerIndexSpecialValues::AnimationBoundary;
	}
}

int32 MarkerCounterSpaceTransform(int32 MaxMarker, int32 Source)
{
	return MaxMarker - 1 - Source;
}

void AdvanceMarkerBackwards(int32& Marker, FName MarkerToFind, bool bLooping, const TArray<FAnimSyncMarker>& AuthoredSyncMarkers, const UMirrorDataTable* MirrorTable)
{
	int32 MaxIterations = AuthoredSyncMarkers.Num();
	const int32 MarkerMax = AuthoredSyncMarkers.Num();
	int32 Counter = MarkerCounterSpaceTransform(MarkerMax, Marker);
	while ((MarkerOrMirroredName(AuthoredSyncMarkers[Marker].MarkerName, MirrorTable) != MarkerToFind) && (--MaxIterations >= 0))
	{
		if ((Marker == 0) && !bLooping)
		{
			break;
		}
		Counter = (Counter + 1) % MarkerMax;
		Marker = MarkerCounterSpaceTransform(MarkerMax, Counter);
	}

	if (!AuthoredSyncMarkers.IsValidIndex(Marker) || (MarkerOrMirroredName(AuthoredSyncMarkers[Marker].MarkerName, MirrorTable) != MarkerToFind))
	{
		Marker = MarkerIndexSpecialValues::AnimationBoundary;
	}
}

bool MarkerMatchesPosition(const UAnimSequence* Sequence, int32 MarkerIndex, FName CorrectMarker, const UMirrorDataTable* MirrorTable)
{
	checkf(MarkerIndex != MarkerIndexSpecialValues::Uninitialized, TEXT("Uninitialized marker supplied to MarkerMatchesPosition. Anim: %s Expecting marker %s (Added to help debug Jira OR-9675)"), *Sequence->GetName(), *CorrectMarker.ToString());
	return MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary || CorrectMarker == MarkerOrMirroredName(Sequence->AuthoredSyncMarkers[MarkerIndex].MarkerName, MirrorTable);
}

void UAnimSequence::ValidateCurrentPosition(const FMarkerSyncAnimPosition& Position, bool bPlayingForwards, bool bLooping, float&CurrentTime, FMarkerPair& PreviousMarker, FMarkerPair& NextMarker, const UMirrorDataTable* MirrorTable) const
{
	if (bPlayingForwards)
	{
		// Ensure previous marker matches the desired previous marker given a name.
		if (!MarkerMatchesPosition(this, PreviousMarker.MarkerIndex, Position.PreviousMarkerName, MirrorTable))
		{
			AdvanceMarkerForwards(PreviousMarker.MarkerIndex, Position.PreviousMarkerName, bLooping, AuthoredSyncMarkers, MirrorTable);

			// Ensure next marker comes after the recently updated previous marker.
			{
				NextMarker.MarkerIndex = (PreviousMarker.MarkerIndex + 1);

				// If needed, loop back or stop at end boundary.
				if (NextMarker.MarkerIndex >= AuthoredSyncMarkers.Num())
				{
					NextMarker.MarkerIndex = bLooping ? NextMarker.MarkerIndex % AuthoredSyncMarkers.Num() : MarkerIndexSpecialValues::AnimationBoundary;
				}
			}
		}

		// Ensure next marker matches the desired next marker given a name.
		if (!MarkerMatchesPosition(this, NextMarker.MarkerIndex, Position.NextMarkerName, MirrorTable))
		{
			AdvanceMarkerForwards(NextMarker.MarkerIndex, Position.NextMarkerName, bLooping, AuthoredSyncMarkers, MirrorTable);
		}
	}
	else
	{
		// Ensure next marker matches the desired next marker given a name.
		if (!MarkerMatchesPosition(this, NextMarker.MarkerIndex, Position.NextMarkerName, MirrorTable))
		{
			AdvanceMarkerBackwards(NextMarker.MarkerIndex, Position.NextMarkerName, bLooping, AuthoredSyncMarkers, MirrorTable);

			// Ensure previous marker comes before the recently updated next marker.
			if (NextMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary || (NextMarker.MarkerIndex == 0 && bLooping))
			{
				PreviousMarker.MarkerIndex = AuthoredSyncMarkers.Num() - 1;
			}
			else
			{
				PreviousMarker.MarkerIndex = NextMarker.MarkerIndex - 1;
			}
		}
		
		// Ensure previous marker matches the desired previous marker given a name.
		if (!MarkerMatchesPosition(this, PreviousMarker.MarkerIndex, Position.PreviousMarkerName, MirrorTable))
		{
			AdvanceMarkerBackwards(PreviousMarker.MarkerIndex, Position.PreviousMarkerName, bLooping, AuthoredSyncMarkers, MirrorTable);
		}
	}

	checkSlow(MarkerMatchesPosition(this, PreviousMarker.MarkerIndex, Position.PreviousMarkerName, MirrorTable));
	checkSlow(MarkerMatchesPosition(this, NextMarker.MarkerIndex, Position.NextMarkerName, MirrorTable));

	// Only reset position if we found valid markers. Otherwise stay where we are to not pop.
	if ((PreviousMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary) && (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary))
	{
		CurrentTime = GetCurrentTimeFromMarkers(PreviousMarker, NextMarker, Position.PositionBetweenMarkers);
	}
}

bool UAnimSequence::ShouldUseRawDataForPoseExtraction(const FBoneContainer& RequiredBones, const FAnimExtractContext& ExtractionContext) const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	return ShouldUseRawDataForPoseExtraction_Lockless(RequiredBones, ExtractionContext);
}

bool UAnimSequence::ShouldUseRawDataForPoseExtraction_Lockless(const FBoneContainer& RequiredBones,
	const FAnimExtractContext& ExtractionContext) const
{
	return ShouldUseRawDataForPoseExtraction_Lockless(ExtractionContext) ||
	RequiredBones.GetDisableRetargeting() ||
	RequiredBones.ShouldUseRawData() ||
	RequiredBones.ShouldUseSourceData();
}

bool UAnimSequence::ShouldUseRawDataForPoseExtraction_Lockless(const FAnimExtractContext& ExtractionContext) const
{
	return CanEvaluateRawAnimationData() &&
	(
		GetSkeleton() == nullptr ||
#if WITH_EDITOR
		GForceRawData == 1 ||
		!GetPlatformCompressedData(ExtractionContext).IsValid(this) ||
#endif // WITH_EDITOR
		(GetSkeletonVirtualBoneGuid() != GetSkeleton()->GetVirtualBoneGuid())
	);
}

void UAnimSequence::EvaluateAttributes(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext, bool bUseRawData) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EvaluateAttributes);

	const FBoneContainer& RequiredBones = OutAnimationPoseData.GetPose().GetBoneContainer();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();
	const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(GetSkeleton(), RequiredBones.GetSkeletonAsset());

#if WITH_EDITOR
	if (bUseRawData)
	{
		ValidateModel();
		for (const FAnimatedBoneAttribute& Attribute : DataModelInterface->GetAttributes())
		{
			const int32 AttributeBoneIndex = Attribute.Identifier.GetBoneIndex();
			const int32 RemappedSkeletonBoneIndex = SkeletonRemapping.IsValid() ? SkeletonRemapping.GetTargetSkeletonBoneIndex(AttributeBoneIndex) : AttributeBoneIndex;
			const FCompactPoseBoneIndex PoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonIndex(RemappedSkeletonBoneIndex);
			// Only add attribute if the, optionally remapped, bone it is targeting exists in the currently evaluated set of bones
			if(PoseBoneIndex.IsValid())
			{
				UE::Anim::Attributes::GetAttributeValue(OutAttributes, PoseBoneIndex, Attribute, ExtractionContext.CurrentTime);
			}
		}
	}
	else
#endif // WITH_EDITOR
	{
		for (const TPair<FAnimationAttributeIdentifier, FAttributeCurve>& BakedAttribute : AttributeCurves)
		{
			const int32 AttributeBoneIndex = BakedAttribute.Key.GetBoneIndex();
			const int32 RemappedSkeletonBoneIndex = SkeletonRemapping.IsValid() ? SkeletonRemapping.GetTargetSkeletonBoneIndex(AttributeBoneIndex) : AttributeBoneIndex;
			const FCompactPoseBoneIndex PoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonIndex(RemappedSkeletonBoneIndex);
			// Only add attribute if , optionally remapped, bone it is targeting exists in the currently evaluated set of bones
			if(PoseBoneIndex.IsValid())
			{
				UE::Anim::FAttributeId Info(BakedAttribute.Key.GetName(), PoseBoneIndex);
				uint8* AttributePtr = OutAttributes.FindOrAdd(BakedAttribute.Key.GetType(), Info);
				BakedAttribute.Value.EvaluateToPtr(BakedAttribute.Key.GetType(), ExtractionContext.CurrentTime, AttributePtr);
			}
		}
	}
}

#if WITH_EDITOR
void UAnimSequence::OnSetSkeleton(USkeleton* NewSkeleton)
{
	WaitOnExistingCompression(false);
}

void UAnimSequence::SynchronousAnimatedBoneAttributesCompression()
{
	struct FByFramePoseEvalContext
	{
	public:
		FBoneContainer RequiredBones;

		TArray<FBoneIndexType> RequiredBoneIndexArray;

		FByFramePoseEvalContext(const UAnimSequence* InAnimToEval)
			: FByFramePoseEvalContext(InAnimToEval->GetSkeleton())
		{}
		
		FByFramePoseEvalContext(USkeleton* InSkeleton)
		{
			// Initialize RequiredBones for pose evaluation
			RequiredBones.SetUseRAWData(true);

			check(InSkeleton);

			RequiredBoneIndexArray.AddUninitialized(InSkeleton->GetReferenceSkeleton().GetNum());
			for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
			{
				RequiredBoneIndexArray[BoneIndex] = BoneIndex;
			}

			RequiredBones.InitializeTo(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::None), *InSkeleton);
		}
	};
	
	ValidateModel();

	AttributeCurves.Empty();

	// If we are additive, we'll need to sample the base pose (against we're additive) and subtract the attributes from the base ones
	const bool bShouldSampleBasePose = IsValidAdditive() && RefPoseType != ABPT_RefPose;
	if (bShouldSampleBasePose)
	{
		// Select which AnimSequence to sample according to additive type
		const UAnimSequence* BasePoseSequence = (RefPoseType == ABPT_LocalAnimFrame) ? this : RefPoseSeq.Get();
		
		// Behaviour for determining the time to sample the base pose attributes
		auto GetBasePoseTimeToSample = [this, BasePoseSequence](float InTime) -> float
		{
			float BasePoseTime = 0.f;

			if (RefPoseType == ABPT_AnimScaled)
			{
				const float CurrentSequenceLength = GetPlayLength();
				const float Fraction = (CurrentSequenceLength > 0.f) ? FMath::Clamp<float>(InTime / CurrentSequenceLength, 0.f, 1.f) : 0.f;
				BasePoseTime = BasePoseSequence->GetPlayLength() * Fraction;
			}
			else if (RefPoseType == ABPT_AnimFrame || RefPoseType == ABPT_LocalAnimFrame)
			{
				const float Fraction = (BasePoseSequence->GetNumberOfSampledKeys() > 0) ? FMath::Clamp<float>((float)RefFrameIndex / (float)BasePoseSequence->GetNumberOfSampledKeys(), 0.f, 1.f) : 0.f;
				BasePoseTime = BasePoseSequence->GetPlayLength() * Fraction;
			}

			return BasePoseTime;
		};

		const FReferenceSkeleton& RefSkeleton = GetSkeleton()->GetReferenceSkeleton();

		FMemMark Mark(FMemStack::Get());

		// Helper struct to match sample timings with regular additive baking
		FByFramePoseEvalContext EvalContext(this);

		for (const FAnimatedBoneAttribute& AdditiveAttribute : DataModelInterface->GetAttributes())
		{
			ensure(!AttributeCurves.Contains(AdditiveAttribute.Identifier));
			FAttributeCurve& AttributeCurve = AttributeCurves.Add(AdditiveAttribute.Identifier);

			if (const FAnimatedBoneAttribute* RefPoseAttributePtr = BasePoseSequence->GetDataModel()->FindAttribute(AdditiveAttribute.Identifier))
			{
				AttributeCurve.SetScriptStruct(AdditiveAttribute.Identifier.GetType());
				const FAnimatedBoneAttribute& RefAttribute = *RefPoseAttributePtr;
				for (int32 KeyIndex = 0; KeyIndex < NumberOfSampledKeys; ++KeyIndex)
				{
					const double CurrentFrameTime = PlatformTargetFrameRate.Default.AsSeconds(KeyIndex);
					UE::Anim::FStackAttributeContainer AdditiveAttributes;
					uint8* AdditivePtr = AdditiveAttributes.FindOrAdd(AdditiveAttribute.Identifier.GetType(), UE::Anim::FAttributeId(NAME_None, 0,NAME_None));
					AdditiveAttribute.Curve.EvaluateToPtr(AdditiveAttribute.Identifier.GetType(), CurrentFrameTime, AdditivePtr);

					UE::Anim::FStackAttributeContainer RefAttributes;
					uint8* RefPtr = RefAttributes.FindOrAdd(RefAttribute.Identifier.GetType(), UE::Anim::FAttributeId(NAME_None, 0, NAME_None));
					RefAttribute.Curve.EvaluateToPtr(RefAttribute.Identifier.GetType(), GetBasePoseTimeToSample(CurrentFrameTime), RefPtr);

					UE::Anim::Attributes::ConvertToAdditive(RefAttributes, AdditiveAttributes);

					AttributeCurve.AddKey(CurrentFrameTime, (void*)AdditivePtr);
				}
			}
			else
			{
				AttributeCurve = AdditiveAttribute.Curve;
			}
		}
	}
	else
	{
		for (const FAnimatedBoneAttribute& Attribute : DataModelInterface->GetAttributes())
		{
			// Do something with the attributes
			ensure(!AttributeCurves.Contains(Attribute.Identifier));

			FAttributeCurve& BakedCurve = AttributeCurves.Add(Attribute.Identifier);
			BakedCurve = Attribute.Curve;
		}
	}

	for (TPair<FAnimationAttributeIdentifier, FAttributeCurve>& BakedAttribute : AttributeCurves)
	{
		BakedAttribute.Value.RemoveRedundantKeys();
	}
}

void UAnimSequence::MoveAttributesToModel()
{
	USkeleton* TargetSkeleton = GetSkeleton();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (TargetSkeleton && PerBoneCustomAttributeData.Num())
	{
		WaitOnExistingCompression();

		IAnimationDataController::FScopedBracket Bracket(Controller, LOCTEXT("MoveAttributesToModel", "Moving legacy Custom Attributes to Model"));

		for (const FCustomAttributePerBoneData& PerBoneData : PerBoneCustomAttributeData)
		{
			const FName BoneName = TargetSkeleton->GetReferenceSkeleton().GetBoneName(PerBoneData.BoneTreeIndex);
			if (BoneName != NAME_None)
			{
				auto ProcessCustomAttribute = [BoneName, this](const FCustomAttribute& Attribute)
				{
					switch (static_cast<EVariantTypes>(Attribute.VariantType))
					{
					case EVariantTypes::Float:
					{
						FAnimationAttributeIdentifier Identifier = UAnimationAttributeIdentifierExtensions::CreateAttributeIdentifier(this, Attribute.Name, BoneName, FFloatAnimationAttribute::StaticStruct());

						if (Controller->AddAttribute(Identifier))
						{
							TArray<int8> TempArray;
							TempArray.SetNumZeroed(FFloatAnimationAttribute::StaticStruct()->GetStructureSize());
							FFloatAnimationAttribute::StaticStruct()->InitializeStruct(TempArray.GetData());

							FFloatAnimationAttribute* TempAttribute = (FFloatAnimationAttribute*)TempArray.GetData();

							for (int32 KeyIndex = 0; KeyIndex < Attribute.Times.Num(); ++KeyIndex)
							{
								const FVariant& VariantValue = Attribute.Values[KeyIndex];
								TempAttribute->Value = VariantValue.GetValue<float>();
								Controller->SetTypedAttributeKey<FFloatAnimationAttribute>(Identifier, Attribute.Times[KeyIndex], *TempAttribute);
							}
						}
						else
						{
							UE_LOGF(LogAnimation, Warning, "Failed to upgrade float attribute %ls for bone %ls", *Attribute.Name.ToString(), *BoneName.ToString());

						}

						break;
					}

					case EVariantTypes::Int32:
					{
						FAnimationAttributeIdentifier Identifier = UAnimationAttributeIdentifierExtensions::CreateAttributeIdentifier(this, Attribute.Name, BoneName, FIntegerAnimationAttribute::StaticStruct());
						if (Controller->AddAttribute(Identifier))
						{
							TArray<int8> TempArray;
							TempArray.SetNumZeroed(FIntegerAnimationAttribute::StaticStruct()->GetStructureSize());
							FIntegerAnimationAttribute::StaticStruct()->InitializeStruct(TempArray.GetData());

							FIntegerAnimationAttribute* TempAttribute = (FIntegerAnimationAttribute*)TempArray.GetData();

							for (int32 KeyIndex = 0; KeyIndex < Attribute.Times.Num(); ++KeyIndex)
							{
								const FVariant& VariantValue = Attribute.Values[KeyIndex];
								TempAttribute->Value = VariantValue.GetValue<int32>();
								Controller->SetTypedAttributeKey<FIntegerAnimationAttribute>(Identifier, Attribute.Times[KeyIndex], *TempAttribute);
							}
						}
						else
						{
							UE_LOGF(LogAnimation, Warning, "Failed to upgrade integer attribute %ls for bone %ls", *Attribute.Name.ToString(), *BoneName.ToString());
						}

						break;
					}

					case EVariantTypes::String:
					{
						FAnimationAttributeIdentifier Identifier = UAnimationAttributeIdentifierExtensions::CreateAttributeIdentifier(this, Attribute.Name, BoneName, FStringAnimationAttribute::StaticStruct());
						if (Controller->AddAttribute(Identifier))
						{
							TArray<int8> TempArray;
							TempArray.SetNumZeroed(FStringAnimationAttribute::StaticStruct()->GetStructureSize());
							FStringAnimationAttribute::StaticStruct()->InitializeStruct(TempArray.GetData());

							FStringAnimationAttribute*TempAttribute = (FStringAnimationAttribute*)TempArray.GetData();
							for (int32 KeyIndex = 0; KeyIndex < Attribute.Times.Num(); ++KeyIndex)
							{
								const FVariant& VariantValue = Attribute.Values[KeyIndex];
								TempAttribute->Value = VariantValue.GetValue<FString>();
								Controller->SetTypedAttributeKey<FStringAnimationAttribute>(Identifier, Attribute.Times[KeyIndex], *TempAttribute);
							}
						}
						else
						{
							UE_LOGF(LogAnimation, Warning, "Failed to upgrade string attribute %ls for bone %ls", *Attribute.Name.ToString(), *BoneName.ToString());
						}
						break;
					}

					default:
					{
						ensureMsgf(false, TEXT("Invalid data variant type for custom attribute, only int32, float and FString are currently supported"));
						break;
					}
					}
				};

				for (const FCustomAttribute& Attribute : PerBoneData.Attributes)
				{
					ProcessCustomAttribute(Attribute);
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Failed to upgrade custom attributes for bone index %i as no such bone index existing in the Skeleton %ls", PerBoneData.BoneTreeIndex, *TargetSkeleton->GetName());
			}
		}

		PerBoneCustomAttributeData.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

void UAnimSequence::AdvanceMarkerPhaseAsFollower(const FMarkerTickContext& Context, float DeltaRemaining, bool bLooping, float& CurrentTime, FMarkerPair& PreviousMarker, FMarkerPair& NextMarker, const UMirrorDataTable* MirrorTable) const
{
	const bool bPlayingForwards = DeltaRemaining >= 0.f;

	// Ensures the sequence's markers match the sync start position.
	ValidateCurrentPosition(Context.GetMarkerSyncStartPosition(), bPlayingForwards, bLooping, CurrentTime, PreviousMarker, NextMarker, MirrorTable);

	if (bPlayingForwards)
	{
		int32 PassedMarkersIndex = 0;

		// Advance all next markers to follow markers passed by leader and update previous markers accordingly.
		do
		{
			// They are no more markers ahead.
			if (NextMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
			{
				check(!bLooping || Context.GetMarkerSyncEndPosition().NextMarkerName == NAME_None); // You shouldn't have an end of anim marker if looping
				CurrentTime = FMath::Min(CurrentTime + DeltaRemaining, GetPlayLength());
				break;
			}
			// Find markers passed by group leader.
			else if (PassedMarkersIndex < Context.MarkersPassedThisTick.Num())
			{
				PreviousMarker.MarkerIndex = NextMarker.MarkerIndex; 

				checkSlow(NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary);
				
				// Advance our next marker to match marker passed by leader.
				{
					const FPassedMarker& MarkerPassedByLeader = Context.MarkersPassedThisTick[PassedMarkersIndex];
					
					AdvanceMarkerForwards(NextMarker.MarkerIndex, MarkerPassedByLeader.PassedMarkerName, bLooping, AuthoredSyncMarkers, MirrorTable);

					// Ensure that any left over delta is handled in last iteration.
					if (NextMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
					{
						DeltaRemaining = MarkerPassedByLeader.DeltaTimeWhenPassed;
					}
				}
				
				++PassedMarkersIndex;
			}
		} while (PassedMarkersIndex < Context.MarkersPassedThisTick.Num());

		// Get sync position after group leader was ticked.
		const FMarkerSyncAnimPosition& LeaderEndPosition = Context.GetMarkerSyncEndPosition();

		// Ensure next marker is a boundary, if the group leader's next marker was one.
		if (LeaderEndPosition.NextMarkerName == NAME_None)
		{
			NextMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
		}
		
		// Ensure next marker matches leader's next marker after tick.
		if (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary && Context.MarkersPassedThisTick.Num() > 0)
		{
			PreviousMarker.MarkerIndex = NextMarker.MarkerIndex;
			
			AdvanceMarkerForwards(NextMarker.MarkerIndex, LeaderEndPosition.NextMarkerName, bLooping, AuthoredSyncMarkers, MirrorTable);
		}

		// Validation
		if (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary)
		{
			check(MarkerOrMirroredName(AuthoredSyncMarkers[NextMarker.MarkerIndex].MarkerName, MirrorTable) == LeaderEndPosition.NextMarkerName);
		}

		// End Validation
		// Only reset position if we found valid markers. Otherwise stay where we are to not pop.
		if ((PreviousMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary) && (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary))
		{
			CurrentTime = GetCurrentTimeFromMarkers(PreviousMarker, NextMarker, LeaderEndPosition.PositionBetweenMarkers);
		}
	}
	else
	{
		int32 PassedMarkersIndex = 0;

		// Advance all previous markers to follow markers passed by leader and update next markers accordingly.
		do
		{
			// They are no more markers ahead.
			if (PreviousMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
			{
				check(!bLooping || Context.GetMarkerSyncEndPosition().PreviousMarkerName == NAME_None); // You shouldn't have an end of anim marker if looping.
				CurrentTime = FMath::Max(CurrentTime + DeltaRemaining, 0.f);
				break;
			}
			// Find markers passed by group leader.
			else if (PassedMarkersIndex < Context.MarkersPassedThisTick.Num())
			{
				NextMarker.MarkerIndex = PreviousMarker.MarkerIndex;
				
				checkSlow(PreviousMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary);

				// Advance previous marker matches marker passed by leader
				{
					const FPassedMarker& MarkerPassedByLeader = Context.MarkersPassedThisTick[PassedMarkersIndex];

					AdvanceMarkerBackwards(PreviousMarker.MarkerIndex, MarkerPassedByLeader.PassedMarkerName, bLooping, AuthoredSyncMarkers, MirrorTable);

					// Ensure that any left over delta is handled in last iteration.
					if (PreviousMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
					{
						DeltaRemaining = MarkerPassedByLeader.DeltaTimeWhenPassed;
					}
					
					++PassedMarkersIndex;
				}
			}
		} while (PassedMarkersIndex < Context.MarkersPassedThisTick.Num());

		// Get sync position after group leader was ticked.
		const FMarkerSyncAnimPosition& LeaderEndPosition = Context.GetMarkerSyncEndPosition();

		// Ensure previous marker is a boundary, if the group leader's was one.
		if (LeaderEndPosition.PreviousMarkerName == NAME_None)
		{
			PreviousMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
		}
		
		// Ensure previous marker match leader's previous marker after tick
		if (PreviousMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary && Context.MarkersPassedThisTick.Num() > 0)
		{
			NextMarker.MarkerIndex = PreviousMarker.MarkerIndex;
			
			AdvanceMarkerBackwards(PreviousMarker.MarkerIndex, LeaderEndPosition.PreviousMarkerName, bLooping, AuthoredSyncMarkers, MirrorTable);
		}

		// Validation
		if (PreviousMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary)
		{
			check(AuthoredSyncMarkers[PreviousMarker.MarkerIndex].MarkerName == LeaderEndPosition.PreviousMarkerName);
		}

		// End Validation
		// Only reset position if we found valid markers. Otherwise stay where we are to not pop.
		if ((PreviousMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary) && (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary))
		{
			CurrentTime = GetCurrentTimeFromMarkers(PreviousMarker, NextMarker, LeaderEndPosition.PositionBetweenMarkers);
		}
	}
}

void UAnimSequence::GetMarkerIndicesForTime(float CurrentTime, bool bLooping, const TArray<FName>& ValidMarkerNames, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker) const
{
	const int LoopModStart = bLooping ? -1 : 0;
	const int LoopModEnd = bLooping ? 2 : 1;

	OutPrevMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
	OutPrevMarker.TimeToMarker = -CurrentTime;
	OutNextMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
	OutNextMarker.TimeToMarker = GetPlayLength() - CurrentTime;

	for (int32 LoopMod = LoopModStart; LoopMod < LoopModEnd; ++LoopMod)
	{
		const float LoopModTime = LoopMod * GetPlayLength();
		for (int Idx = 0; Idx < AuthoredSyncMarkers.Num(); ++Idx)
		{
			const FAnimSyncMarker& Marker = AuthoredSyncMarkers[Idx];
			if (ValidMarkerNames.Contains(Marker.MarkerName))
			{
				const float MarkerTime = Marker.Time + LoopModTime;
				if (MarkerTime < CurrentTime)
				{
					OutPrevMarker.MarkerIndex = Idx;
					OutPrevMarker.TimeToMarker = MarkerTime - CurrentTime;
				}
				else if (MarkerTime >= CurrentTime)
				{
					OutNextMarker.MarkerIndex = Idx;
					OutNextMarker.TimeToMarker = MarkerTime - CurrentTime;
					break; // Done
				}
			}
		}
		if (OutNextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary)
		{
			break; // Done
		}
	}
}

FMarkerSyncAnimPosition UAnimSequence::GetMarkerSyncPositionFromMarkerIndicies(int32 PrevMarker, int32 NextMarker, float CurrentTime, const UMirrorDataTable* MirrorTable) const
{
	FMarkerSyncAnimPosition SyncPosition;
	float PrevTime, NextTime;

	// Get previous marker's time and name.
	if (PrevMarker != MarkerIndexSpecialValues::AnimationBoundary && ensureAlwaysMsgf(AuthoredSyncMarkers.IsValidIndex(PrevMarker),
		TEXT("%s - MarkerCount: %d, PrevMarker : %d, NextMarker: %d, CurrentTime : %0.2f"), *GetFullName(), AuthoredSyncMarkers.Num(), PrevMarker, NextMarker, CurrentTime))
	{
		PrevTime = AuthoredSyncMarkers[PrevMarker].Time;
		SyncPosition.PreviousMarkerName = MarkerOrMirroredName(AuthoredSyncMarkers[PrevMarker].MarkerName, MirrorTable); 
	}
	else
	{
		PrevTime = 0.f;
	}

	// Get next marker's time and name.
	if (NextMarker != MarkerIndexSpecialValues::AnimationBoundary && ensureAlwaysMsgf(AuthoredSyncMarkers.IsValidIndex(NextMarker),
		TEXT("%s - MarkerCount: %d, PrevMarker : %d, NextMarker: %d, CurrentTime : %0.2f"), *GetFullName(), AuthoredSyncMarkers.Num(), PrevMarker, NextMarker, CurrentTime))
	{
		NextTime = AuthoredSyncMarkers[NextMarker].Time;
		SyncPosition.NextMarkerName = MarkerOrMirroredName(AuthoredSyncMarkers[NextMarker].MarkerName, MirrorTable);
	}
	else
	{
		NextTime = GetPlayLength();
	}

	// Account for looping
	if(PrevTime > NextTime)
	{
		PrevTime = (PrevTime > CurrentTime) ? PrevTime - GetPlayLength() : PrevTime;
		NextTime = (NextTime < CurrentTime) ? NextTime + GetPlayLength() : NextTime;
	}
	else if (PrevTime > CurrentTime)
	{
		CurrentTime += GetPlayLength();
	}

	if (PrevTime == NextTime)
	{
		PrevTime -= GetPlayLength();
	}

	check(NextTime > PrevTime);

	// Store the encoded current time position as a ratio between markers
	SyncPosition.PositionBetweenMarkers = (CurrentTime - PrevTime) / (NextTime - PrevTime);
	return SyncPosition;
}

float UAnimSequence::GetCurrentTimeFromMarkers(FMarkerPair& PrevMarker, FMarkerPair& NextMarker, float PositionBetweenMarkers) const
{
	// Query marker times, or start and end boundary times, respectively.
	float PrevTime = (PrevMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary) ? AuthoredSyncMarkers[PrevMarker.MarkerIndex].Time : 0.f;
	float NextTime = (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary) ? AuthoredSyncMarkers[NextMarker.MarkerIndex].Time : GetPlayLength();

	// Account for looping
	if (PrevTime >= NextTime)
	{
		PrevTime -= GetPlayLength(); 
	}

	// Compute current time given start and end marker times.
	float CurrentTime = PrevTime + PositionBetweenMarkers * (NextTime - PrevTime);

	// Compute time to reach each marker.
	PrevMarker.TimeToMarker = PrevTime - CurrentTime;
	NextMarker.TimeToMarker = NextTime - CurrentTime;

	// Account for looping while playing backwards.
	if (CurrentTime < 0.f)
	{
		CurrentTime += GetPlayLength();
	}
	
	CurrentTime = FMath::Clamp<float>(CurrentTime, 0, GetPlayLength());

	return CurrentTime;
}

void UAnimSequence::GetMarkerIndicesForPosition(
	const FMarkerSyncAnimPosition& SyncPosition, bool bLooping, FMarkerPair& OutPrevMarker, 
	FMarkerPair& OutNextMarker, float& OutCurrentTime, const UMirrorDataTable* MirrorTable) const
{
	auto GetMarkerName = [MirrorTable](const FAnimSyncMarker& SyncMarker)->FName
	{
		return MarkerOrMirroredName(SyncMarker.MarkerName, MirrorTable);
	};
	
	// If we're not looping, assume we're playing a transition and we need to stay where we are.
	// Also do this if we have no usable SyncPosition.
	if (!bLooping || (SyncPosition.PreviousMarkerName == NAME_None && SyncPosition.NextMarkerName == NAME_None))
	{
		OutPrevMarker.MarkerIndex = INDEX_NONE;
		OutNextMarker.MarkerIndex = INDEX_NONE;

		for (int32 Idx = 0; Idx<AuthoredSyncMarkers.Num(); Idx++)
		{
			const FAnimSyncMarker& SyncMarker = AuthoredSyncMarkers[Idx];
			const float MarkerTime = SyncMarker.Time;

			// Match the position's previous marker name, and store its index.
			if (OutCurrentTime > MarkerTime && GetMarkerName(SyncMarker) == SyncPosition.PreviousMarkerName)
			{
				OutPrevMarker.MarkerIndex = Idx;
				OutPrevMarker.TimeToMarker = MarkerTime - OutCurrentTime;
			}
			// Match the position's next marker name, and store its index. By this point we should have found the previous marker index so we can stop searching.
			else if (OutCurrentTime < MarkerTime && GetMarkerName(SyncMarker) == SyncPosition.NextMarkerName)
			{
				OutNextMarker.MarkerIndex = Idx;
				OutNextMarker.TimeToMarker = MarkerTime - OutCurrentTime;
				break;
			}
		}

		ensureMsgf(OutCurrentTime >= 0.f && OutCurrentTime <= GetPlayLength(), TEXT("Current time inside of GetMarkerIndicesForPosition is out of range %.3f of 0.0 to %.3f\n    Sequence: %s"), OutCurrentTime, GetPlayLength(), *GetFullName());
		return;
	}

	// Handle case where the position's previous marker is the start boundary.
	if (SyncPosition.PreviousMarkerName == NAME_None)
	{
		// Make output prev marker index be the start boundary.
		OutPrevMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;

		// Our position's next marker should never be the end boundary, otherwise we dont have any sync markers at all.
		check(SyncPosition.NextMarkerName != NAME_None);

		// Find next marker index.
		for (int32 Idx = 0; Idx < AuthoredSyncMarkers.Num(); ++Idx)
		{
			const FAnimSyncMarker& Marker = AuthoredSyncMarkers[Idx];
			if (GetMarkerName(Marker) == SyncPosition.NextMarkerName)
			{
				OutNextMarker.MarkerIndex = Idx;
				OutCurrentTime = GetCurrentTimeFromMarkers(OutPrevMarker, OutNextMarker, SyncPosition.PositionBetweenMarkers);
				return;
			}
		}
		
		// Should have found a marker above!
		checkf(false, TEXT("Next Marker not found in GetMarkerIndicesForPosition. Anim: %s Expecting marker %s (Added to help debug Jira OR-9675)"), *GetName(), *SyncPosition.NextMarkerName.ToString());
	}

	// Handle case where the position's next marker is the end boundary.
	if (SyncPosition.NextMarkerName == NAME_None)
	{
		// Make output next marker index be the end boundary.
		OutNextMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;

		// Our position's previous marker should never be the start boundary, otherwise we dont have any sync markers at all.
		check(SyncPosition.PreviousMarkerName != NAME_None);

		// Find previous marker index.
		for (int32 Idx = AuthoredSyncMarkers.Num() - 1; Idx >= 0; --Idx)
		{
			const FAnimSyncMarker& Marker = AuthoredSyncMarkers[Idx];
			if (GetMarkerName(Marker) == SyncPosition.PreviousMarkerName)
			{
				OutPrevMarker.MarkerIndex = Idx;
				OutCurrentTime = GetCurrentTimeFromMarkers(OutPrevMarker, OutNextMarker, SyncPosition.PositionBetweenMarkers);
				return;
			}
		}
		
		// Should have found a marker above!
		checkf(false, TEXT("Previous Marker not found in GetMarkerIndicesForPosition. Anim: %s Expecting marker %s (Added to help debug Jira OR-9675)"), *GetName(), *SyncPosition.PreviousMarkerName.ToString());
	}

	float DiffToCurrentTime = FLT_MAX;
	const float CurrentInputTime  = OutCurrentTime;

	// Handle case for looping and sync position not being on either boundary.
	for (int32 PrevMarkerIdx = 0; PrevMarkerIdx < AuthoredSyncMarkers.Num(); ++PrevMarkerIdx)
	{
		const FAnimSyncMarker& PrevMarker = AuthoredSyncMarkers[PrevMarkerIdx];

		// We have matched the position's previous marker name.
		if (GetMarkerName(PrevMarker) == SyncPosition.PreviousMarkerName)
		{
			const int32 EndMarkerSearchStart = PrevMarkerIdx + 1;
			const int32 EndCount = bLooping ? AuthoredSyncMarkers.Num() + EndMarkerSearchStart : AuthoredSyncMarkers.Num();
			
			for (int32 NextMarkerCount = EndMarkerSearchStart; NextMarkerCount < EndCount; ++NextMarkerCount)
			{
				const int32 NextMarkerIdx = NextMarkerCount % AuthoredSyncMarkers.Num();

				// We have matched the position's next marker name.
				if (GetMarkerName(AuthoredSyncMarkers[NextMarkerIdx]) == SyncPosition.NextMarkerName)
				{
					float NextMarkerTime = AuthoredSyncMarkers[NextMarkerIdx].Time;

					// Handle case where we need to loop to get to be able to get to the next marker.
					bool bLooped = false;
					if (NextMarkerTime < PrevMarker.Time)
					{
						NextMarkerTime += GetPlayLength();
						bLooped = true;
					}

					// Get current time based of sync position.
					float ThisCurrentTime = PrevMarker.Time + SyncPosition.PositionBetweenMarkers * (NextMarkerTime - PrevMarker.Time);

					// Find marker indices closest to input time position.
					float ThisDiff = FMath::Abs(ThisCurrentTime - CurrentInputTime);
					if (ThisDiff < DiffToCurrentTime)
					{
						DiffToCurrentTime = ThisDiff;
						OutPrevMarker.MarkerIndex = PrevMarkerIdx;
						OutNextMarker.MarkerIndex = NextMarkerIdx;
						OutCurrentTime = GetCurrentTimeFromMarkers(OutPrevMarker, OutNextMarker, SyncPosition.PositionBetweenMarkers);
					}
					else if (bLooped)
					{
						// If we looped, we extended our next marker past the end of the sequence
						// This means that there are two points we need to test:
						//   - The one that lands near the end of the sequence (possibly overshooting/looping around)
						//   - The one that lands near the start of the sequence (possibly undershooting/looping around)
						// 
						// We tested the first one above, now test the second
						ThisCurrentTime -= GetPlayLength();

						ThisDiff = FMath::Abs(ThisCurrentTime - CurrentInputTime);
						if (ThisDiff < DiffToCurrentTime)
						{
							DiffToCurrentTime = ThisDiff;
							OutPrevMarker.MarkerIndex = PrevMarkerIdx;
							OutNextMarker.MarkerIndex = NextMarkerIdx;
							OutCurrentTime = GetCurrentTimeFromMarkers(OutPrevMarker, OutNextMarker, SyncPosition.PositionBetweenMarkers);
						}
					}

					// This marker test is done, move onto next one.
					break;
				}
			}

			// If we get here and we haven't found a match and we are not looping then there 
			// is no point running the rest of the loop set up something as relevant as we can and carry on
			if (OutPrevMarker.MarkerIndex == MarkerIndexSpecialValues::Uninitialized)
			{
				//Find nearest previous marker that is earlier than our current time
				DiffToCurrentTime = OutCurrentTime - PrevMarker.Time;
				int32 PrevMarkerToUse = PrevMarkerIdx + 1;
				while (DiffToCurrentTime > 0.f && PrevMarkerToUse < AuthoredSyncMarkers.Num())
				{
					DiffToCurrentTime = OutCurrentTime - AuthoredSyncMarkers[PrevMarkerToUse].Time;
					++PrevMarkerToUse;
				}
				OutPrevMarker.MarkerIndex = PrevMarkerToUse - 1;	// We always go one past the marker we actually want to use
				
				OutNextMarker.MarkerIndex = -1;						// This goes to minus one as the very fact we are here means
																	// that there is no next marker to use
				OutCurrentTime = GetCurrentTimeFromMarkers(OutPrevMarker, OutNextMarker, SyncPosition.PositionBetweenMarkers);
				break; // no need to keep searching, we are done
			}
		}
	}
	
	// Should have found a markers above!
	checkf(OutPrevMarker.MarkerIndex != MarkerIndexSpecialValues::Uninitialized, TEXT("Prev Marker not found in GetMarkerIndicesForPosition. Anim: %s Expecting marker %s (Added to help debug Jira OR-9675)"), *GetName(), *SyncPosition.PreviousMarkerName.ToString());
	checkf(OutNextMarker.MarkerIndex != MarkerIndexSpecialValues::Uninitialized, TEXT("Next Marker not found in GetMarkerIndicesForPosition. Anim: %s Expecting marker %s (Added to help debug Jira OR-9675)"), *GetName(), *SyncPosition.NextMarkerName.ToString());
}

float UAnimSequence::GetFirstMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition) const
{
	if ((InMarkerSyncGroupPosition.PreviousMarkerName == NAME_None) || (InMarkerSyncGroupPosition.NextMarkerName == NAME_None))
	{
		return 0.f;
	}

	for (int32 PrevMarkerIdx = 0; PrevMarkerIdx < AuthoredSyncMarkers.Num()-1; PrevMarkerIdx++)
	{
		const FAnimSyncMarker& PrevMarker = AuthoredSyncMarkers[PrevMarkerIdx];
		const FAnimSyncMarker& NextMarker = AuthoredSyncMarkers[PrevMarkerIdx+1];
		if ((PrevMarker.MarkerName == InMarkerSyncGroupPosition.PreviousMarkerName) && (NextMarker.MarkerName == InMarkerSyncGroupPosition.NextMarkerName))
		{
			return FMath::Lerp(PrevMarker.Time, NextMarker.Time, InMarkerSyncGroupPosition.PositionBetweenMarkers);
		}
	}

	return 0.f;
}

float UAnimSequence::GetNextMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition, const float& StartingPosition) const
{
	if ((InMarkerSyncGroupPosition.PreviousMarkerName == NAME_None) || (InMarkerSyncGroupPosition.NextMarkerName == NAME_None))
	{
		return StartingPosition;
	}

	for (int32 PrevMarkerIdx = 0; PrevMarkerIdx < AuthoredSyncMarkers.Num() - 1; PrevMarkerIdx++)
	{
		const FAnimSyncMarker& PrevMarker = AuthoredSyncMarkers[PrevMarkerIdx];
		const FAnimSyncMarker& NextMarker = AuthoredSyncMarkers[PrevMarkerIdx + 1];

		if (NextMarker.Time < StartingPosition)
		{
			continue;
		}

		if ((PrevMarker.MarkerName == InMarkerSyncGroupPosition.PreviousMarkerName) && (NextMarker.MarkerName == InMarkerSyncGroupPosition.NextMarkerName))
		{
			const float FoundTime = FMath::Lerp(PrevMarker.Time, NextMarker.Time, InMarkerSyncGroupPosition.PositionBetweenMarkers);
			if (FoundTime < StartingPosition)
			{
				continue;
			}
			return FoundTime;
		}
	}

	return StartingPosition;
}

float UAnimSequence::GetPrevMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition, const float& StartingPosition) const
{
	if ((InMarkerSyncGroupPosition.PreviousMarkerName == NAME_None) || (InMarkerSyncGroupPosition.NextMarkerName == NAME_None) || (AuthoredSyncMarkers.Num() < 2))
	{
		return StartingPosition;
	}

	for (int32 PrevMarkerIdx = AuthoredSyncMarkers.Num() - 2; PrevMarkerIdx >= 0; PrevMarkerIdx--)
	{
		const FAnimSyncMarker& PrevMarker = AuthoredSyncMarkers[PrevMarkerIdx];
		const FAnimSyncMarker& NextMarker = AuthoredSyncMarkers[PrevMarkerIdx + 1];

		if (PrevMarker.Time > StartingPosition)
		{
			continue;
		}

		if ((PrevMarker.MarkerName == InMarkerSyncGroupPosition.PreviousMarkerName) && (NextMarker.MarkerName == InMarkerSyncGroupPosition.NextMarkerName))
		{
			const float FoundTime = FMath::Lerp(PrevMarker.Time, NextMarker.Time, InMarkerSyncGroupPosition.PositionBetweenMarkers);
			if (FoundTime > StartingPosition)
			{
				continue;
			}
			return FoundTime;
		}
	}

	return StartingPosition;
}

void UAnimSequence::EnableRootMotionSettingFromMontage(bool bInEnableRootMotion, const ERootMotionRootLock::Type InRootMotionRootLock)
{
	if (!bRootMotionSettingsCopiedFromMontage)
	{
		bEnableRootMotion = bInEnableRootMotion;
		RootMotionRootLock = InRootMotionRootLock;
		bRootMotionSettingsCopiedFromMontage = true;
	}
}

bool UAnimSequence::GetEnableRootMotionSettingFromMontage() const
{
	return bRootMotionSettingsCopiedFromMontage;
}

bool UAnimSequence::IsCompressedDataValid() const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	return GetCompressedData_Internal().IsValid(this);
}

bool UAnimSequence::IsBoneCompressedDataValid() const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	return GetCompressedData_Internal().IsBoneDataValid(this);
}

bool UAnimSequence::IsCurveCompressedDataValid() const
{
	UE_COMPRESSED_DATA_READ_SCOPE(this);
	return GetCompressedData_Internal().IsCurveDataValid(this);
}

#if WITH_EDITOR
void UAnimSequence::EnsureValidRawDataGuid()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IsDataModelValid() && !RawDataGuid.IsValid())
	{
		RawDataGuid = DataModelInterface->GenerateGuid();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAnimSequence::PopulateModel()
{
	check(!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject));
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const int32 NumKeys = FMath::Max(NumberOfKeys, 2);
	const float PlayLength = SequenceLength;
	// Reset target framerate to current sampling frame rate
	TargetFrameRate = SamplingFrameRate;
	PlatformTargetFrameRate = SamplingFrameRate;
	const FFrameRate FrameRate = SamplingFrameRate;
	const FRawCurveTracks CurveData = RawCurveData;
	const TArray<FRawAnimSequenceTrack> SequenceTracks = SourceRawAnimationData_DEPRECATED.Num() && SourceRawAnimationData_DEPRECATED.Num() == RawAnimationData.Num() ?
		SourceRawAnimationData_DEPRECATED :
		RawAnimationData;
	const TArray<FName> TempAnimationTrackNames = AnimationTrackNames;
	
	const bool bCurveOnlySequence = RawAnimationData.Num() == 0;
	const bool bValidNumberOfKeys = NumberOfKeys >= 1 || bCurveOnlySequence;
	const float ExpectedPlayLength = (float)SamplingFrameRate.AsSeconds(FMath::Max(NumberOfKeys - 1, 1));
	const bool bValidSequenceLength = FMath::IsNearlyEqual(ExpectedPlayLength, SequenceLength, UE_KINDA_SMALL_NUMBER) || bCurveOnlySequence;
	const bool bValidNumberOfTrackArrayEntries = AnimationTrackNames.Num() == RawAnimationData.Num();
	const bool bValidAnimationData = SourceRawAnimationData_DEPRECATED.Num() == 0 || SourceRawAnimationData_DEPRECATED.Num() == RawAnimationData.Num();
	const bool bValidFrameRate = SamplingFrameRate.IsValid() && (SamplingFrameRate.AsInterval() >= 0.f);
	const bool bValidCurveData = RawCurveData.FloatCurves.Num() >= 1;

	if (!((bValidNumberOfTrackArrayEntries || bValidCurveData) && bValidAnimationData && bValidFrameRate && bValidNumberOfKeys && bValidSequenceLength))
	{
		UE_LOGF(LogAnimation, Warning, "Possible invalid animation data during model generation %ls - \n\tNumber of Keys: %i\n\tPlay length: %f\n\tNumber of Tracks: %i\n\tFrameRate: %ls\n\tNumber of float Curves: %i", *GetName(), NumKeys, PlayLength, RawAnimationData.Num(), *SamplingFrameRate.ToPrettyText().ToString(), RawCurveData.FloatCurves.Num());
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (RefPoseSeq && RefPoseSeq->HasAnyFlags(EObjectFlags::RF_NeedPostLoad))
	{
		RefPoseSeq->ConditionalPostLoad();
	}

	Controller->OpenBracket(LOCTEXT("UAnimSequence::PopulateModel_Bracket", "Generating Animation Model Data from Animation Sequence"));

	Controller->SetFrameRate(FrameRate);

	const FFrameTime FrameTime = FrameRate.AsFrameTime(bCurveOnlySequence ? PlayLength : ExpectedPlayLength);
	const FFrameNumber RoundedFrameNumber = FrameTime.RoundToFrame();
	ensureAlways(RoundedFrameNumber.Value == FMath::Max((NumKeys - 1), 1));
	Controller->SetNumberOfFrames(RoundedFrameNumber);	

	const USkeleton* TargetSkeleton = GetSkeleton();
	UE::Anim::CopyCurveDataToModel(CurveData, TargetSkeleton, *Controller);
	
	const int32 NumTracks = SequenceTracks.Num();
	for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
	{
		if (SequenceTracks[TrackIndex].ContainsNaN())
		{
			continue;
		}
		
		if (Controller->AddBoneCurve(TempAnimationTrackNames[TrackIndex]))
		{
			auto GenerateUniformKeys = [NumKeys](auto& Keys, auto IdentityKey)
			{
				// Convert track keys to be uniform
				if (Keys.Num() == 0)
				{
					// set all to identity
					for (int32 Index = 0; Index < NumKeys; ++Index)
					{
						Keys.Add(IdentityKey);
					}
				}
				else if (Keys.Num() == 1)
				{
					// set all to single key
					const auto KeyZero = Keys[0];
					for (int32 Index = 0; Index < (NumKeys - 1); ++Index)
					{
						Keys.Add(KeyZero);
					}
				}

				ensure(Keys.Num() == NumKeys);
			};

			TArray<FVector3f> PosKeys = SequenceTracks[TrackIndex].PosKeys;
			GenerateUniformKeys(PosKeys, FVector3f::ZeroVector);

			TArray<FQuat4f> RotKeys = SequenceTracks[TrackIndex].RotKeys;
			GenerateUniformKeys(RotKeys, FQuat4f::Identity);

			TArray<FVector3f> ScaleKeys = SequenceTracks[TrackIndex].ScaleKeys;
			GenerateUniformKeys(ScaleKeys, FVector3f::OneVector);

			Controller->SetBoneTrackKeys(TempAnimationTrackNames[TrackIndex], PosKeys, RotKeys, ScaleKeys);
		}
	}

	DeleteDeprecatedRawAnimationData();
	
	Controller->NotifyPopulated();
	
	Controller->CloseBracket();
}

void UAnimSequence::OnModelModified(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& Payload)
{
	UAnimSequenceBase::OnModelModified(NotifyType, Model, Payload);

	static const bool GenerateNewGUID = true;
	static const bool RegenerateGUID = false;

	auto UpdateRawDataGuid = [this, Model](bool bForceNewRawDataGuid)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RawDataGuid = bForceNewRawDataGuid ? FGuid::NewGuid() : Model->GenerateGuid();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FlagDependentAnimationsAsRawDataOnly();
		UpdateDependentStreamingAnimations();
	};

	auto HandleTrackDataChanged = [this, UpdateRawDataGuid](bool bWasReset = false)
	{
		if (NotifyCollector.IsNotWithinBracket())
		{
			CalculateNumberOfSampledKeys();
			UpdateRawDataGuid(RegenerateGUID);
			ClearAllCompressionData();
			if(!bWasReset)
			{
				BeginCacheDerivedDataForCurrentPlatform();
			}
		}
	};

	auto HandleNumberOfFramesChanged = [this](FFrameNumber NewNumberOfFrames, FFrameNumber OldNumberOfFrames, FFrameNumber Frame0, FFrameNumber Frame1)
	{
		// Do not handle changes during model population, or undo-redo (notifies are transacted so will be restored/set, so only handle in case of, initial, user interaction)
		if (bPopulatingDataModel || GIsTransacting)
		{
			return;
		}
		const FFrameRate& ModelFrameRate = DataModelInterface->GetFrameRate();

		const float NewLength = ModelFrameRate.AsSeconds(NewNumberOfFrames);
		const float T0 = ModelFrameRate.AsSeconds(Frame0);
		const float T1 = ModelFrameRate.AsSeconds(Frame1);

		if (NewNumberOfFrames > OldNumberOfFrames)
		{
			const float InsertTime = T0;
			const float Duration = T1 - T0;

			for (FAnimSyncMarker& Marker : AuthoredSyncMarkers)
			{
				float CurrentTime = Marker.Time;

				// when insert, we only care about start time
				// if it's later than start time
				if (CurrentTime >= InsertTime)
				{
					CurrentTime += Duration;
				}

				Marker.Time = FMath::Clamp(CurrentTime, 0.f, NewLength);
			}
		}
		else if (NewNumberOfFrames < OldNumberOfFrames)
		{
			const float StartRemoveTime = T0;
			const float EndRemoveTime = T1;

			// Total time value for frames that were removed
			const float Duration = T1 - T0;

			for (FAnimSyncMarker& Marker : AuthoredSyncMarkers)
			{
				float CurrentTime = Marker.Time;

				if (CurrentTime >= StartRemoveTime && CurrentTime <= EndRemoveTime)
				{
					CurrentTime = StartRemoveTime;
				}
				else if (CurrentTime > EndRemoveTime)
				{
					CurrentTime -= Duration;
				}

				Marker.Time = FMath::Clamp(CurrentTime, 0.f, NewLength);
			}
		}
	};

	bool bShouldMarkPackageDirty = !FUObjectThreadContext::Get().IsRoutingPostLoad && NotifyType != EAnimDataModelNotifyType::BracketOpened;

	switch (NotifyType)
	{
		case EAnimDataModelNotifyType::SequenceLengthChanged:
		{
			const FSequenceLengthChangedPayload& TypedPayload = Payload.GetPayload<FSequenceLengthChangedPayload>();

			const FFrameNumber OldNumberOfFrames = TypedPayload.PreviousNumberOfFrames;
			const FFrameNumber CurrentNumberOfFrames = Model->GetNumberOfFrames();
			HandleNumberOfFramesChanged(CurrentNumberOfFrames, OldNumberOfFrames, TypedPayload.Frame0, TypedPayload.Frame1);
						
			if (NotifyCollector.IsNotWithinBracket())
			{
				HandleTrackDataChanged();
			}

			break;
		}

		case EAnimDataModelNotifyType::FrameRateChanged:
		{
			const FFrameRateChangedPayload& TypedPayload = Payload.GetPayload<FFrameRateChangedPayload>();
			
			if (NotifyCollector.IsNotWithinBracket())
			{
				HandleTrackDataChanged();
			}

			break;
		}

		case EAnimDataModelNotifyType::Populated:
		{
			PlatformTargetFrameRate = Model->GetFrameRate();
				
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			RawCurveData.Empty();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			if (NotifyCollector.IsNotWithinBracket())
			{
				HandleTrackDataChanged();				
			}	
			break;
		}
		
		case EAnimDataModelNotifyType::BracketClosed:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				const auto LengthChangingNotifies = { EAnimDataModelNotifyType::SequenceLengthChanged, EAnimDataModelNotifyType::FrameRateChanged, EAnimDataModelNotifyType::Reset };
				const auto ResamplingNotifies = { EAnimDataModelNotifyType::TrackAdded, EAnimDataModelNotifyType::TrackChanged, EAnimDataModelNotifyType::TrackRemoved,  EAnimDataModelNotifyType::Populated };
				const auto RecompressNotifies = { EAnimDataModelNotifyType::CurveAdded, EAnimDataModelNotifyType::CurveChanged, EAnimDataModelNotifyType::CurveRemoved, EAnimDataModelNotifyType::CurveFlagsChanged, EAnimDataModelNotifyType::CurveScaled,
				EAnimDataModelNotifyType::AttributeAdded, EAnimDataModelNotifyType::AttributeChanged, EAnimDataModelNotifyType::AttributeRemoved, EAnimDataModelNotifyType::SkeletonChanged };

				bShouldMarkPackageDirty = NotifyCollector.WasDataModified();
				
				if (NotifyCollector.Contains(LengthChangingNotifies) || NotifyCollector.Contains(ResamplingNotifies))
				{
					CalculateNumberOfSampledKeys();
				}

				if (NotifyCollector.Contains(LengthChangingNotifies) || NotifyCollector.Contains(ResamplingNotifies) || NotifyCollector.Contains(RecompressNotifies))
				{
					const bool bWasModelReset = NotifyCollector.Contains(EAnimDataModelNotifyType::Reset);
					UpdateRawDataGuid(bWasModelReset ? GenerateNewGUID : RegenerateGUID);
					ClearAllCompressionData();
					HandleTrackDataChanged(bWasModelReset);
				}
			}

			break;
		}

		case EAnimDataModelNotifyType::BracketOpened:
		{
			break;
		}

		case EAnimDataModelNotifyType::Reset:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				CalculateNumberOfSampledKeys();
				UpdateRawDataGuid(GenerateNewGUID);
				ClearAllCompressionData();
				BeginCacheDerivedDataForCurrentPlatform();
			}

			break;
		}
		
		case EAnimDataModelNotifyType::TrackAdded:
		case EAnimDataModelNotifyType::TrackChanged:
		case EAnimDataModelNotifyType::TrackRemoved:		
		{
			HandleTrackDataChanged();				
			break;
		}

		case EAnimDataModelNotifyType::CurveAdded:
		case EAnimDataModelNotifyType::CurveChanged:
		case EAnimDataModelNotifyType::CurveRemoved:
		case EAnimDataModelNotifyType::CurveFlagsChanged:
		case EAnimDataModelNotifyType::CurveScaled:
		{
			ClearAllCompressionData();

			if (NotifyCollector.IsNotWithinBracket())
			{						
				UpdateRawDataGuid(RegenerateGUID);
				BeginCacheDerivedDataForCurrentPlatform();
			}

			break;
		}

		case EAnimDataModelNotifyType::AttributeAdded:
		case EAnimDataModelNotifyType::AttributeChanged:
		case EAnimDataModelNotifyType::AttributeRemoved:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				UpdateRawDataGuid(RegenerateGUID);
				BeginCacheDerivedDataForCurrentPlatform();
			}
			break;
		}

		case EAnimDataModelNotifyType::CurveColorChanged:
		case EAnimDataModelNotifyType::CurveCommentChanged:
		{
			break;
		}

		case EAnimDataModelNotifyType::CurveRenamed:
		{
			const FCurveRenamedPayload& TypedPayload = Payload.GetPayload<FCurveRenamedPayload>();
			UpdateCompressedCurveName(TypedPayload.Identifier.CurveName, TypedPayload.NewIdentifier.CurveName);
			break;
		}

		case EAnimDataModelNotifyType::SkeletonChanged:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				HandleTrackDataChanged();
			}
			break;
		}

		default:
		{
			checkf(false, TEXT("Missing case statement for animation model notify type"));
			break;
		}
	}

	if (NotifyCollector.IsNotWithinBracket())
	{
		if (bShouldMarkPackageDirty)
		{
			MarkPackageDirty();
		}
	}
	else if (bShouldMarkPackageDirty)
	{
		NotifyCollector.MarkDataModified();
	}
}

void UAnimSequence::CalculateNumberOfSampledKeys()
{
	if (PlatformTargetFrameRate.Default == DataModelInterface->GetFrameRate())
	{
		NumberOfSampledKeys = DataModelInterface->GetNumberOfKeys();
		NumberOfSampledFrames = DataModelInterface->GetNumberOfFrames();
	}
	else
	{
		const FFrameNumber ModelNumberOfFrames = DataModelInterface->GetNumberOfFrames();
		const FFrameTime ResampledFrameTime = FFrameRate::TransformTime(ModelNumberOfFrames, DataModelInterface->GetFrameRate(), PlatformTargetFrameRate.Default);
		ensureMsgf(FMath::IsNearlyZero(ResampledFrameTime.GetSubFrame()), TEXT("Incompatible resampling frame rate for animation sequence %s, frame remainder of %1.8f"), *GetName(), ResampledFrameTime.GetSubFrame());

		NumberOfSampledFrames = ResampledFrameTime.FloorToFrame().Value;
		NumberOfSampledKeys = NumberOfSampledFrames + 1;
	}
}

FIoHash UAnimSequence::GetDerivedDataKeyHash(const ITargetPlatform* TargetPlatform) const
{
	thread_local uint32 CachedCurrentPlatformHash = INDEX_NONE;
	const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	if (CachedCurrentPlatformHash == INDEX_NONE)
	{
		CachedCurrentPlatformHash = GetTypeHash(RunningPlatform);
	}
	
	const uint32 PlatformHash = TargetPlatform ? GetTypeHash(TargetPlatform) : CachedCurrentPlatformHash;

	const FIoHash* HashPtr = nullptr;
	{
		FReadScopeLock ScopeLock(HashCacheLock);
		HashPtr = PlatformHashToKeyHash.Find(PlatformHash);
	}
		
	return HashPtr ? *HashPtr : CreateDerivedDataKeyHash(TargetPlatform ? TargetPlatform : RunningPlatform);
}

// Opt-in binary path: archive-based key generation (skips intermediate FString).
static bool GAnimSequenceUseBinaryDDCKey = false;
static FAutoConsoleVariableRef CVarAnimSequenceUseBinaryDDCKey(
	TEXT("a.AnimSequence.UseBinaryDDCKey"),
	GAnimSequenceUseBinaryDDCKey,
	TEXT("When true, UAnimSequence uses archive-based DDC key generation (skips intermediate FString allocation). Default false uses the legacy CreateDerivedDataKeyString path."));

FIoHash UAnimSequence::CreateDerivedDataKeyHash(const ITargetPlatform* TargetPlatform) const
{
	check(TargetPlatform);

	const uint32 PlatformHash = GetTypeHash(TargetPlatform);
	FIoHash Hash;

	if (GAnimSequenceUseBinaryDDCKey)
	{
		// Write directly to the hasher.
		FMemoryHasherBlake3 Writer;
		if (!CreateDerivedDataKey(TargetPlatform, Writer))
		{
			return FIoHash::Zero;
		}
		Hash = Writer.Finalize();
	}
	else
	{
		// Legacy: build readable key string then hash.
		FString Ret = CreateDerivedDataKeyString(TargetPlatform);
		if (Ret.IsEmpty())
		{
			return FIoHash::Zero;
		}
		FMemoryHasherBlake3 Writer;
		Writer << Ret;
		Hash = Writer.Finalize();
	}

	{
		FWriteScopeLock ScopeLock(HashCacheLock);
		// Verifying key (changing)
		if (FIoHash* StoredHash = PlatformHashToKeyHash.Find(PlatformHash))
		{
			if (*StoredHash != Hash)
			{
				checkf(!IsRunningCookCommandlet(), TEXT("Hash for %s has changed from %s to %s"), *GetPathName(), *LexToString(*StoredHash), *LexToString(Hash));
			}
		}
		
		// Store platform-to-hash pair 
		PlatformHashToKeyHash.Add(PlatformHash, Hash);
	}

	return Hash;
}

bool UAnimSequence::CreateDerivedDataKey(const ITargetPlatform* TargetPlatform, FArchive& Output) const
{
    const USkeleton* CurrentSkeleton = GetSkeleton();
    if (CurrentSkeleton == nullptr)
    {
        UE_LOGF(LogAnimation, Warning, "Animation Compression request for %ls failed, Skeleton == nullptr.", *GetName());
        return false;
    }

    if (BoneCompressionSettings == nullptr || CurveCompressionSettings == nullptr || VariableFrameStrippingSettings == nullptr)
    {
        UE_LOGF(LogAnimation, Warning, "Animation Compression request for %ls failed, BoneCompressionSettings == nullptr || CurveCompressionSettings == nullptr || VariableFrameStrippingSettings == nullptr.", *GetName());
        return false;
    }

    const bool bIsValidAdditive = IsValidAdditive();

    bool bPerformFrameStripping = GPerformFrameStripping == 1;
    bool bPerformFrameStrippingOnOddNumberedFrames = GPerformFrameStrippingOddFramedAnimations == 1;
    {
        if (UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(TargetPlatform->IniPlatformName()))
        {
            int32 CVarPlatformFrameStrippingValue = 0;
            if (DeviceProfile->GetConsolidatedCVarValue(StripFrameCVarName, CVarPlatformFrameStrippingValue))
            {
                bPerformFrameStripping = CVarPlatformFrameStrippingValue == 1;
            }

            int32 CVarPlatformOddAnimFrameStrippingValue = 0;
            if (DeviceProfile->GetConsolidatedCVarValue(OddFrameStripStrippingCVarName, CVarPlatformOddAnimFrameStrippingValue))
            {
                bPerformFrameStrippingOnOddNumberedFrames = CVarPlatformOddAnimFrameStrippingValue == 1;
            }
        }
        bPerformFrameStripping = ShouldPerformStripping(bPerformFrameStripping, bPerformFrameStrippingOnOddNumberedFrames);
    }

    // FArchive::operator<< takes a non-const ref; pass-by-value gives us a writable local.
    auto Write = [&Output](auto Value) { Output << Value; };

    Write(CompressCommandletVersion);
    Write(GetDataModel()->GenerateGuid());
    Write(CurrentSkeleton->GetGuid());
    Write(CurrentSkeleton->GetVirtualBoneGuid());

    // Additive type bundle — encode as raw bytes; absent additive guid emits nothing (length differentiates).
    Write(bIsValidAdditive ? (uint8)AdditiveAnimType : (uint8)0);
    Write(bIsValidAdditive ? (uint8)RefPoseType : (uint8)0);
    Write(RefFrameIndex);
    if (bIsValidAdditive && RefPoseSeq)
    {
        Write(RefPoseSeq->GetDataModel()->GenerateGuid());
    }

    Write(CompressionErrorThresholdScale);
    Write(bPerformFrameStripping);

    BoneCompressionSettings->PopulateDDCKey(UE::Anim::Compression::FAnimDDCKeyArgs(*this, TargetPlatform), Output);
    CurveCompressionSettings->PopulateDDCKey(Output);
    VariableFrameStrippingSettings->PopulateDDCKey(UE::Anim::Compression::FAnimDDCKeyArgs(*this, TargetPlatform), Output);

    if (bIsValidAdditive)
    {
        // Additive sequences are compressed in re-targeted space, as such we need to include the re-targeting transforms in our key
        for (FTransform RetargetTransform : GetRetargetTransforms())
        {
            Write(RetargetTransform);
        }
    }

    // Include sockets since they can impact precision requirements
    for (const USkeletalMeshSocket* Socket : CurrentSkeleton->Sockets)
    {
        Write(Socket->BoneName);
    }

    if (FCompressibleAnimData::GUseHighFidelityAdditiveCurveCompression)
    {
        Write(FCompressibleAnimData::GUseHighFidelityAdditiveCurveCompression);
    }

    const FFrameRate FrameRate = UE::Anim::Compression::GetCompressionFrameRate(*this, TargetPlatform);
    Write(FrameRate.Numerator);
    Write(FrameRate.Denominator);
    Write(UE::Anim::Compression::AnimationCompressionVersionString);

    return true;
}

FString UAnimSequence::CreateDerivedDataKeyString(const ITargetPlatform* TargetPlatform) const
{
	const USkeleton* CurrentSkeleton = GetSkeleton();
	if (CurrentSkeleton == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "Animation Compression request for %ls failed, Skeleton == nullptr.", *GetName());
		return FString();
	}

	if (BoneCompressionSettings == nullptr || CurveCompressionSettings == nullptr || VariableFrameStrippingSettings == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "Animation Compression request for %ls failed, BoneCompressionSettings == nullptr || CurveCompressionSettings == nullptr || VariableFrameStrippingSettings == nullptr.", *GetName());
		return FString();
	}

	const bool bIsValidAdditive = IsValidAdditive();
	const char AdditiveType = bIsValidAdditive ? NibbleToTChar(AdditiveAnimType) : '0';
	const char RefType = bIsValidAdditive ? NibbleToTChar(RefPoseType) : '0';

	bool bPerformFrameStripping = GPerformFrameStripping == 1;
	bool bPerformFrameStrippingOnOddNumberedFrames = GPerformFrameStrippingOddFramedAnimations == 1;
	{
		if (UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(TargetPlatform->IniPlatformName()))
		{
			int32 CVarPlatformFrameStrippingValue = 0;
			if (DeviceProfile->GetConsolidatedCVarValue(StripFrameCVarName, CVarPlatformFrameStrippingValue))
			{
				bPerformFrameStripping = CVarPlatformFrameStrippingValue == 1;
			}
			
			int32 CVarPlatformOddAnimFrameStrippingValue = 0;
			if (DeviceProfile->GetConsolidatedCVarValue(OddFrameStripStrippingCVarName, CVarPlatformOddAnimFrameStrippingValue))
			{
				bPerformFrameStrippingOnOddNumberedFrames = CVarPlatformOddAnimFrameStrippingValue == 1;
			}
		}

		bPerformFrameStripping = ShouldPerformStripping(bPerformFrameStripping, bPerformFrameStrippingOnOddNumberedFrames);
	}

	FArcToHexString ArcToHexString;

	float TempThreshold = CompressionErrorThresholdScale;
	ArcToHexString.Ar << TempThreshold;
	ArcToHexString.Ar << bPerformFrameStripping;
	BoneCompressionSettings->PopulateDDCKey(UE::Anim::Compression::FAnimDDCKeyArgs(*this, TargetPlatform), ArcToHexString.Ar);
	CurveCompressionSettings->PopulateDDCKey(ArcToHexString.Ar);
	VariableFrameStrippingSettings->PopulateDDCKey(UE::Anim::Compression::FAnimDDCKeyArgs(*this, TargetPlatform), ArcToHexString.Ar);

	if (bIsValidAdditive)
	{
		// Additive sequences are compressed in re-targeted space, as such we need to include the re-targeting transforms in our key
		const TArray<FTransform>& RetargetTransforms = GetRetargetTransforms();
		for (FTransform RetargetTransform : RetargetTransforms)	// Copy the transform to allow us to use the non-const << operator below
		{
			ArcToHexString.Ar << RetargetTransform;
		}
	}

	// Include sockets since they can impact precision requirements
	for (const USkeletalMeshSocket* Socket : CurrentSkeleton->Sockets)
	{
		FName BoneName = Socket->BoneName;
		ArcToHexString.Ar << BoneName;
	}
	
	if (FCompressibleAnimData::GUseHighFidelityAdditiveCurveCompression)
	{
		ArcToHexString.Ar << FCompressibleAnimData::GUseHighFidelityAdditiveCurveCompression;
	}

	const FFrameRate FrameRate = UE::Anim::Compression::GetCompressionFrameRate(*this, TargetPlatform);

	FString Ret = FString::Printf(TEXT("%i_%s%s%s_%c%c%i_%s_%s_%i_%i_%s"),
		CompressCommandletVersion,
		*GetDataModel()->GenerateGuid().ToString(),
		*CurrentSkeleton->GetGuid().ToString(),
		*CurrentSkeleton->GetVirtualBoneGuid().ToString(),
		AdditiveType,
		RefType,
		RefFrameIndex,
		(bIsValidAdditive && RefPoseSeq) ? *RefPoseSeq->GetDataModel()->GenerateGuid().ToString() : TEXT("NoAdditiveGuid"),
		*ArcToHexString.MakeString(),
		FrameRate.Numerator,
		FrameRate.Denominator,
		*UE::Anim::Compression::AnimationCompressionVersionString
	);
	
	return Ret;
}

void UAnimSequence::ValidateCompressionSettings()
{
	// Ensure that there are valid compression settings
	if (BoneCompressionSettings == nullptr || !BoneCompressionSettings->AreSettingsValid())
	{
		BoneCompressionSettings = FAnimationUtils::GetDefaultAnimationBoneCompressionSettings();
	}	
	if (CurveCompressionSettings == nullptr || !CurveCompressionSettings->AreSettingsValid())
	{
		CurveCompressionSettings = FAnimationUtils::GetDefaultAnimationCurveCompressionSettings();
	}
	if (VariableFrameStrippingSettings == nullptr)
	{
		VariableFrameStrippingSettings = FAnimationUtils::GetDefaultVariableFrameStrippingSettings();
	}
}

bool UAnimSequence::CanBeCompressed() const
{
	const UPackage* Package = GetPackage();
	return Package && (Package->GetHasBeenEndLoaded() || Package->GetLoadedPath().IsEmpty()) && !Package->HasAnyPackageFlags(PKG_Cooked);
}

FIoHash UAnimSequence::BeginCacheDerivedData(const ITargetPlatform* TargetPlatform)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimSequence_BeginCacheDerivedData);

	check(TargetPlatform != nullptr);
	check(IsInGameThread());
	check(!FPlatformProperties::RequiresCookedData());
	
	check(!IsUnreachable());
	
	if(bBlockCompressionRequests)
	{
		UE_LOGF(LogAnimation, Warning, "Animation Compression request for %ls was blocked, bBlockCompressionRequests == true.", *GetName());
		return FIoHash::Zero;
	}

	// Wait for any in-flight requests to finish
	// Once a compression request finishes, it might modify the compression settings below and the key hash as well
	UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation({ this, RefPoseSeq });

	if (!CanBeCompressed())
	{
		return FIoHash::Zero;
	}

	ValidateCompressionSettings();

	// Make sure all our required dependencies are loaded, we need them to compute the KeyHash
	FAnimationUtils::EnsureAnimSequenceLoaded(*this);
		
	const FIoHash KeyHash = CreateDerivedDataKeyHash(TargetPlatform);

	UE_COMPRESSED_DATA_WRITE_SCOPE(this);
	UE::TScopeLock LockGuard(CacheTasksMutex);
	FCompressedAnimSequence* TargetData = nullptr;
	{
		checkf(!CacheTasksByKeyHash.Contains(KeyHash), TEXT("[%s] Hash %s still/already has task in-flight"), *GetPathName(), *LexToString(KeyHash));
		// Early out if not valid, has already been cached, or has an inflight task running 
		if (KeyHash.IsZero() || CacheTasksByKeyHash.Contains(KeyHash) || DataByPlatformKeyHash.Contains(KeyHash))
		{
			return KeyHash;
		}

		if (RequiresResidency(KeyHash))
		{
			UE_LOGF(LogAnimation, Warning, "Animation Compression request for %ls was blocked, Residency.", *GetName());
			return FIoHash::Zero;
		}

		TargetData = DataByPlatformKeyHash.Emplace(KeyHash, MakeUnique<FCompressedAnimSequence>()).Get();
	}

	const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();	
	if (TargetPlatform == RunningPlatform)
	{
		CurrentPlatformData = nullptr;
	}

	// Reset the target compressed data, to mark it invalid
	check(TargetData);
	TargetData->Reset();

	bool bPerformFrameStripping  = GPerformFrameStripping == 1;
	{
		bool bPerformFrameStrippingOnOddNumberedFrames = GPerformFrameStrippingOddFramedAnimations == 1;
		if (UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(TargetPlatform->IniPlatformName()))
		{
			int32 CVarPlatformFrameStrippingValue = 0;
			if (DeviceProfile->GetConsolidatedCVarValue(StripFrameCVarName, CVarPlatformFrameStrippingValue))
			{
				bPerformFrameStripping = CVarPlatformFrameStrippingValue == 1;
			}
			
			int32 CVarPlatformOddAnimFrameStrippingValue = 0;
			if (DeviceProfile->GetConsolidatedCVarValue(OddFrameStripStrippingCVarName, CVarPlatformOddAnimFrameStrippingValue))
			{
				bPerformFrameStrippingOnOddNumberedFrames = CVarPlatformOddAnimFrameStrippingValue == 1;
			}
		}

		bPerformFrameStripping = ShouldPerformStripping(bPerformFrameStripping, bPerformFrameStrippingOnOddNumberedFrames);
	}

	// Always keep AnimatedBoneAttributes in lock-step when requesting compression
	{
		SynchronousAnimatedBoneAttributesCompression();
	}

	if (!CacheTasksByKeyHash.Contains(KeyHash))
	{
		// Data does not exist, need to build it.
		const FCompressibleAnimRef CompressibleData = MakeShared<FCompressibleAnimData, ESPMode::ThreadSafe>(this, bPerformFrameStripping, TargetPlatform);
		
		COOK_STAT(auto Timer = UE::Anim::AnimSequenceCookStats::UsageStats.TimeSyncWork());
		check(CacheTasksByKeyHash.Find(KeyHash) == nullptr);
		CacheTasksByKeyHash.Emplace(KeyHash, MakePimpl<UE::Anim::FAnimationSequenceAsyncCacheTask>(KeyHash, CompressibleData, TargetData, *this, TargetPlatform));		
		COOK_STAT(Timer.TrackCyclesOnly());
	}
	else
    {
		check(false);
    }

	// The compiling manager provides throttling, notification manager, etc... for the asset being built.
	UE::Anim::FAnimSequenceCompilingManager::Get().AddAnimSequences({this});
	
	return KeyHash;
}

bool UAnimSequence::PollCacheDerivedData(const FIoHash& KeyHash) const
{
	if (KeyHash.IsZero())
	{
		return true;
	}

	UE::TScopeLock LockGuard(CacheTasksMutex);
	if (const TPimplPtr<UE::Anim::FAnimationSequenceAsyncCacheTask>* Task = CacheTasksByKeyHash.Find(KeyHash))
	{
		return (*Task)->Poll();
	}

	return false;
}

void UAnimSequence::EndCacheDerivedData(const FIoHash& KeyHash)
{
	if (KeyHash.IsZero())
	{
		return;
	}
	
	UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation({this});
}
	
void UAnimSequence::CacheDerivedDataForPlatform(const ITargetPlatform* TargetPlatform)
{
	const FIoHash KeyHash = BeginCacheDerivedData(TargetPlatform);
	EndCacheDerivedData(KeyHash);
	check(KeyHash.IsZero() || HasCompressedDataForHash(KeyHash));
}

FFrameRate UAnimSequence::GetTargetSamplingFrameRate(const ITargetPlatform* InPlatform) const
{
	return PlatformTargetFrameRate.GetValueForPlatform(*InPlatform->IniPlatformName());
}

void UAnimSequence::BeginCacheDerivedDataForCurrentPlatform()
{
	if (const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform())
	{
		BeginCacheDerivedData(RunningPlatform);
	}
}

void UAnimSequence::CacheDerivedDataForCurrentPlatform()
{
	if (const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform())
	{
		CacheDerivedDataForPlatform(RunningPlatform);
	}
}
				
bool UAnimSequence::IsCompiling() const
{
	UE::TScopeLock LockGuard(CacheTasksMutex);
	return CacheTasksByKeyHash.Num() > 0;
}

void UAnimSequence::ClearAllCompressionData()
{
	{
		// Delete any cache tasks first because the destructor will cancel the cache and build tasks,
		// and drop their pointers to the data.
		{
			UE::TScopeLock LockGuard(CacheTasksMutex);
			for (auto It = CacheTasksByKeyHash.CreateIterator(); It; ++It)
			{
				if (!RequiresResidency(It->Key))
				{
					It.RemoveCurrent();
				}
			}
		}

		UE_COMPRESSED_DATA_WRITE_SCOPE(this);
		for (auto It = DataByPlatformKeyHash.CreateIterator(); It; ++It)
		{
			if (!RequiresResidency(It->Key))
			{
				It.RemoveCurrent();
			}
		}

		CurrentPlatformData = nullptr;
	}
}

void UAnimSequence::ClearCompressionData(const FIoHash& InKeyHash)
{
	check(!InKeyHash.IsZero());
	
	// Check if there are any references left, otherwise clear out
	if(!RequiresResidency(InKeyHash))
	{
		UE_COMPRESSED_DATA_WRITE_SCOPE(this);
		check(DataByPlatformKeyHash.Contains(InKeyHash));
		DataByPlatformKeyHash.Remove(InKeyHash);
		
		const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		if (GetDerivedDataKeyHash(RunningPlatform) == InKeyHash)
		{
			CurrentPlatformData = nullptr;
		}
	}
}

bool UAnimSequence::RequiresResidency(const FIoHash& InKeyHash) const
{
	FReadScopeLock ScopeLock(ResidencyLock);
	return PlatformHashToReferencers.Contains(InKeyHash);
}

bool UAnimSequence::HasResidency(const ITargetPlatform* InPlatform) const
{
	const FIoHash KeyHash = GetDerivedDataKeyHash(InPlatform);
	FReadScopeLock ScopeLock(ResidencyLock);
	return PlatformHashToReferencers.Contains(KeyHash);
}

bool UAnimSequence::HasResidency(uint32 InReferencerHash) const
{
	FReadScopeLock ScopeLock(ResidencyLock);
	return ResidencyReferencerHashes.Contains(InReferencerHash);
}

void UAnimSequence::RequestResidency(const ITargetPlatform* InPlatform, uint32 InReferencerHash)
{
	check(CanBeCompressed() && !HasAnyFlags(RF_NeedPostLoad));
	const FIoHash KeyHash = GetDerivedDataKeyHash(InPlatform);

	TArray<uint32> ReferencerHashes;
	{
		FReadScopeLock ScopeLock(ResidencyLock);
		PlatformHashToReferencers.MultiFind(KeyHash, ReferencerHashes);
	}
	
	// First request for residency
	if (ReferencerHashes.Num() == 0)
	{
		UE_CLOGF(false, LogAnimationCompression, Display, "RequestResidency platform data [KeyHash]: %ls %ls %x %ls", *InPlatform->DisplayName().ToString(), *LexToString(KeyHash), InReferencerHash, *GetPathName());
		BeginCacheForCookedPlatformData(InPlatform);
	}

	if (!ReferencerHashes.Contains(InReferencerHash))
	{
		FWriteScopeLock ScopeLock(ResidencyLock);
		// Store referencer hash alongside platform has at point of request (as data changes during compression can impact platform hash)
		ResidencyReferencerHashes.Add(InReferencerHash, KeyHash);
		PlatformHashToReferencers.Add(KeyHash, InReferencerHash);
	}
}

void UAnimSequence::ReleaseResidency(const ITargetPlatform* InPlatform, uint32 InReferencerHash)
{
	FIoHash KeyHash = FIoHash::Zero;
	TArray<uint32> ReferencerHashes;
	{
		FWriteScopeLock ScopeLock(ResidencyLock);
		KeyHash = ResidencyReferencerHashes.FindChecked(InReferencerHash);

		// Remove entry from both maps
		verify(ResidencyReferencerHashes.Remove(InReferencerHash) == 1);
		verify(PlatformHashToReferencers.Remove(KeyHash, InReferencerHash) == 1);
		
		// Check if there are any references left
		PlatformHashToReferencers.MultiFind(KeyHash, ReferencerHashes);

	
		UE_CLOGF(false, LogAnimationCompression, Display, "ReleaseResidency platform data [KeyHash]: %ls %ls %x %ls", *InPlatform->DisplayName().ToString(), *LexToString(KeyHash), InReferencerHash, *GetPathName());
	}
	
	if (ReferencerHashes.Num() == 0 && !KeyHash.IsZero())
	{
		// If previously this anim sequence was marked as never to be cooked again clear out compressed data without residency references
		if(bShouldClearCompressedData)
		{
			ClearCompressionData(KeyHash);
		}
	}
}
bool UAnimSequence::HasCompressedDataForPlatform(const ITargetPlatform* InPlatform) const
{
	const FIoHash KeyHash = GetDerivedDataKeyHash(InPlatform);
	return HasCompressedDataForHash(KeyHash);
}

bool UAnimSequence::TryCancelAsyncTasks()
{
	UE::TScopeLock LockGuard(CacheTasksMutex);
	for (auto It = CacheTasksByKeyHash.CreateIterator(); It; ++It)
	{
		It->Value->Cancel();
	}

	return CacheTasksByKeyHash.IsEmpty();
}

bool UAnimSequence::WaitForAsyncTasks(float TimeLimitSeconds)
{
	double StartTimeSeconds = FPlatformTime::Seconds();
	UE::TScopeLock LockGuard(CacheTasksMutex);
	for (auto& Pair : CacheTasksByKeyHash)
	{
		// Clamp to 0 as it implies polling
		const float TimeLimit = FMath::Min(0.0f, TimeLimitSeconds - (FPlatformTime::Seconds() - StartTimeSeconds));
		if (!Pair.Value->WaitWithTimeout(TimeLimit))
		{
			return false;
		}
	}
	return true;
}

bool UAnimSequence::IsAsyncTaskComplete() const
{
	UE::TScopeLock LockGuard(CacheTasksMutex);
	bool bAllFinished = true;
	for (auto& Pair : CacheTasksByKeyHash)
	{
		bAllFinished &= Pair.Value->Poll();
	}

	return bAllFinished;
}

void UAnimSequence::FinishAsyncTasks()
{
	const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	const FIoHash CurrentPlatformHash = CreateDerivedDataKeyHash(RunningPlatform);

	UE::TScopeLock LockGuard(CacheTasksMutex);
	const bool bHasInflightTasks = CacheTasksByKeyHash.Num() > 0;
	if (bHasInflightTasks)
	{
		COOK_STAT(auto Timer = UE::Anim::AnimSequenceCookStats::UsageStats.TimeAsyncWait());
		COOK_STAT(Timer.TrackCyclesOnly());
		
		for (auto It = CacheTasksByKeyHash.CreateIterator(); It; ++It)
		{
			// Wait has potential to hit FAnimationSequenceAsyncCacheTask::BuildData or FAnimationSequenceAsyncCacheTask::EndCache which requires a write-lock
			It->Value->Wait();

			UE_COMPRESSED_DATA_WRITE_SCOPE(this);
			FCompressedAnimSequence* TaskData = It->Value->GetTargetCompressedData();
			const bool bCompressedRunningPlatform = It->Key == CurrentPlatformHash && It->Value->GetTargetPlatform() == RunningPlatform;
			auto ResetData = [this, TaskData, It]()
			{
				if(TaskData != nullptr)
				{
					TaskData->Reset();
				}
				DataByPlatformKeyHash.Remove(It->Key);
			};

			if (It->Value.Get()->WasCancelled())
			{
				ResetData();
			}
			else
			{
				if (TaskData && TaskData->IsValid(this, true))
				{
					//This is only safe during sync anim compression
					if (GetSkeleton())
					{
						SetSkeletonVirtualBoneGuid(GetSkeleton()->GetVirtualBoneGuid());
					}
					if (bCompressedRunningPlatform)
					{
						FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
						AssetRegistryModule.Get().AssetTagsFinalized(*this);

						// Cache compressed data ptr for current platform to by-pass TMap lookup requiring data-hash retrieval/generation and locking
						ensure(CurrentPlatformData == nullptr);
						CurrentPlatformData = TaskData;
					}
				
					check(TaskData->IsValid(this, true));
				}
				else
				{
					// Failed to compress
					UE_LOGF(LogAnimationCompression, Display, "Failed to finish async Animation Compression task for %ls, as the generated data is not valid.", *GetName());
					ResetData();
				}
			}

			It.RemoveCurrent();
		}
	}
}

void UAnimSequence::Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority)
{
	UE::TScopeLock LockGuard(CacheTasksMutex);
	for (auto& Pair : CacheTasksByKeyHash)
	{
		Pair.Value->Reschedule(InThreadPool, InPriority);
	}
}
#endif // WITH_EDITOR

/*-----------------------------------------------------------------------------
	AnimNotify& subclasses
-----------------------------------------------------------------------------*/

#if !UE_BUILD_SHIPPING

void GatherAnimSequenceStats(FOutputDevice& Ar)
{
	int32 AnimationKeyFormatNum[AKF_MAX];
	int32 TranslationCompressionFormatNum[ACF_MAX];
	int32 RotationCompressionFormatNum[ACF_MAX];
	int32 ScaleCompressionFormatNum[ACF_MAX];
	FMemory::Memzero( AnimationKeyFormatNum, AKF_MAX * sizeof(int32) );
	FMemory::Memzero( TranslationCompressionFormatNum, ACF_MAX * sizeof(int32) );
	FMemory::Memzero( RotationCompressionFormatNum, ACF_MAX * sizeof(int32) );
	FMemory::Memzero( ScaleCompressionFormatNum, ACF_MAX * sizeof(int32) );

	Ar.Logf( TEXT(" %60s, Frames,NTT,NRT, NT1,NR1, TotTrnKys,TotRotKys,Codec,ResBytes"), TEXT("Sequence Name") );
	int32 GlobalNumTransTracks = 0;
	int32 GlobalNumRotTracks = 0;
	int32 GlobalNumScaleTracks = 0;
	int32 GlobalNumTransTracksWithOneKey = 0;
	int32 GlobalNumRotTracksWithOneKey = 0;
	int32 GlobalNumScaleTracksWithOneKey = 0;
	int32 GlobalApproxCompressedSize = 0;
	int32 GlobalApproxKeyDataSize = 0;
	int32 GlobalNumTransKeys = 0;
	int32 GlobalNumRotKeys = 0;
	int32 GlobalNumScaleKeys = 0;

	for( TObjectIterator<UAnimSequence> It; It; ++It )
	{
		const UAnimSequence* Seq = *It;

		UAnimSequence::FScopedCompressedAnimSequence ScopedCompressedData = Seq->GetCompressedData();
		const FCompressedAnimSequence& CompressedAnimSequence = ScopedCompressedData.Get();
		if (CompressedAnimSequence.CompressedDataStructure == nullptr || !CompressedAnimSequence.BoneCompressionCodec->IsA<UAnimCompress>())
		{
			continue;	// Custom codec we know nothing about, skip it
		}

		int32 NumTransTracks = 0;
		int32 NumRotTracks = 0;
		int32 NumScaleTracks = 0;
		int32 TotalNumTransKeys = 0;
		int32 TotalNumRotKeys = 0;
		int32 TotalNumScaleKeys = 0;
		float TranslationKeySize = 0.0f;
		float RotationKeySize = 0.0f;
		float ScaleKeySize = 0.0f;
		int32 OverheadSize = 0;
		int32 NumTransTracksWithOneKey = 0;
		int32 NumRotTracksWithOneKey = 0;
		int32 NumScaleTracksWithOneKey = 0;

		const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(*CompressedAnimSequence.CompressedDataStructure);

		AnimationFormat_GetStats(
			AnimData,
			NumTransTracks,
			NumRotTracks,
			NumScaleTracks,
			TotalNumTransKeys,
			TotalNumRotKeys,
			TotalNumScaleKeys,
			TranslationKeySize,
			RotationKeySize,
			ScaleKeySize, 
			OverheadSize,
			NumTransTracksWithOneKey,
			NumRotTracksWithOneKey,
			NumScaleTracksWithOneKey);

		GlobalNumTransTracks += NumTransTracks;
		GlobalNumRotTracks += NumRotTracks;
		GlobalNumScaleTracks += NumScaleTracks;
		GlobalNumTransTracksWithOneKey += NumTransTracksWithOneKey;
		GlobalNumRotTracksWithOneKey += NumRotTracksWithOneKey;
		GlobalNumScaleTracksWithOneKey += NumScaleTracksWithOneKey;

		GlobalApproxCompressedSize += Seq->GetApproxCompressedSize();
		GlobalApproxKeyDataSize += (int32)((TotalNumTransKeys * TranslationKeySize) + (TotalNumRotKeys * RotationKeySize) + (TotalNumScaleKeys * ScaleKeySize));

		GlobalNumTransKeys += TotalNumTransKeys;
		GlobalNumRotKeys += TotalNumRotKeys;
		GlobalNumScaleKeys += TotalNumScaleKeys;

		Ar.Logf(TEXT(" %60s, %3i, %3i,%3i,%3i, %3i,%3i,%3i, %10i,%10i,%10i, %s, %d"),
			*Seq->GetName(),
			Seq->GetNumberOfSampledKeys(),
			NumTransTracks, NumRotTracks, NumScaleTracks,
			NumTransTracksWithOneKey, NumRotTracksWithOneKey, NumScaleTracksWithOneKey,
			TotalNumTransKeys, TotalNumRotKeys, TotalNumScaleKeys,
			*FAnimationUtils::GetAnimationKeyFormatString(AnimData.KeyEncodingFormat),
			(int32)It->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal) );
	}
	Ar.Logf( TEXT("======================================================================") );
	Ar.Logf( TEXT("Total Num Tracks: %i trans, %i rot, %i scale, %i trans1, %i rot1, %i scale1"), GlobalNumTransTracks, GlobalNumRotTracks, GlobalNumScaleTracks, GlobalNumTransTracksWithOneKey, GlobalNumRotTracksWithOneKey, GlobalNumScaleTracksWithOneKey  );
	Ar.Logf( TEXT("Total Num Keys: %i trans, %i rot, %i scale"), GlobalNumTransKeys, GlobalNumRotKeys, GlobalNumScaleKeys );

	Ar.Logf( TEXT("Approx Compressed Memory: %i bytes"), GlobalApproxCompressedSize);
	Ar.Logf( TEXT("Approx Key Data Memory: %i bytes"), GlobalApproxKeyDataSize);
}

#endif // !UE_BUILD_SHIPPING


#undef LOCTEXT_NAMESPACE 

