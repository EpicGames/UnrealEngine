// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "AnimationUtils.h"
#include "Animation/AnimNotifyQueue.h"
#include "AnimationRuntime.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/AnimNotifyEndDataContext.h"
#include "Animation/Skeleton.h"
#include "Logging/MessageLog.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Modules/ModuleManager.h"
#include "MathUtil.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

#if WITH_EDITOR
#include "IAnimationDataControllerModule.h"
#include "Modules/ModuleManager.h"
#endif // WITH_EDITOR

#include "Animation/AnimationSettings.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectThreadContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequenceBase)

DEFINE_LOG_CATEGORY(LogAnimMarkerSync);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

#define LOCTEXT_NAMESPACE "AnimSequenceBase"
/////////////////////////////////////////////////////

UAnimSequenceBase::UAnimSequenceBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RateScale(1.0f)
	, bLoop(false)
#if WITH_EDITORONLY_DATA
	, DataModel(nullptr)
	, bPopulatingDataModel(false)
	, Controller(nullptr)
#endif // WITH_EDITORONLY_DATA
{
#if WITH_EDITOR
	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject| EObjectFlags::RF_NeedLoad))
	{
		CreateModel();
		GetController();
	}
#endif // WITH_EDITOR
}

bool UAnimSequenceBase::IsPostLoadThreadSafe() const
{
	return WITH_EDITORONLY_DATA == 0;	// Not thread safe in editor because new objects can be constructed on upgrade and the skeleton can be modified
}

#if WITH_EDITORONLY_DATA
void UAnimSequenceBase::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);

	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/Engine.AnimDataModel")));

	// We need to declare all types that can be returned from UE::Anim::DataModel::IAnimationDataModels::FindClassForAnimationAsset(UAnimSequenceBase*);
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/AnimationData.AnimationSequencerDataModel")));

	// We can call Controller->CreateModel, which can add objects, so add every ControllerClass as something we can construct.
	// THe caller will add on all recursively constructable classes
	UClass* AnimationControllerClass = UAnimationDataController::StaticClass();
	for (TObjectIterator<UClass> Iter; Iter; ++Iter)
	{
		if ((*Iter)->ImplementsInterface(AnimationControllerClass))
		{
			OutConstructClasses.Add(FTopLevelAssetPath(*Iter));
		}
	}
}
#endif

void UAnimSequenceBase::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	const FRawCurveTracks& CurveData = GetCurveData();
	const UStruct* Struct = CurveData.StaticStruct();
	if (ensure(Struct))
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Struct->GetStructureSize());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CurveData.FloatCurves.GetAllocatedSize());
	}
}

void UAnimSequenceBase::PostLoad()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		LLM_SCOPE(ELLMTag::Animation);

		auto PreloadSkeleton = [this]()
		{
			if (USkeleton* MySkeleton = GetSkeleton())
			{
				MySkeleton->ConditionalPreload();
				MySkeleton->ConditionalPostLoad();
			}
		};

		if(ShouldDataModelBeValid())
		{
		    const UClass* TargetDataModelClass = UE::Anim::DataModel::IAnimationDataModels::FindClassForAnimationAsset(this);
		    const bool bRequiresModelCreation = DataModelInterface == nullptr || DataModelInterface.GetObject()->GetClass() != TargetDataModelClass ||  GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::ReintroduceAnimationDataModelInterface;

		    TScriptInterface<IAnimationDataModel> CachedDataModelInterface = DataModelInterface;
		    
		    const bool bRequiresModelPopulation = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::IntroducingAnimationDataModel;
		    PRAGMA_DISABLE_DEPRECATION_WARNINGS
		    checkf(bRequiresModelPopulation || DataModel != nullptr || DataModelInterface != nullptr, TEXT("Invalid Animation Sequence base state, no data model found past upgrade object version. AnimSequenceBase:%s"), *GetPathName());
		    PRAGMA_ENABLE_DEPRECATION_WARNINGS
    
		    // Construct a new IAnimationDataModel instance
		    if(bRequiresModelCreation)
		    {
			    CreateModel();
		    }

			PreloadSkeleton();
			
		    ValidateModel();
			if (UObject* DataModelObject = DataModelInterface.GetObject())
			{
				DataModelObject->ConditionalPostLoad();
				DataModelObject->ConditionalPostLoadSubobjects();
			}
		    GetController();
		    BindToModelModificationEvent();		    

		    if (bRequiresModelPopulation || bRequiresModelCreation)
		    {
			    const bool bDoNotTransactAction = false;
    
		    	Controller->OpenBracket(LOCTEXT("UAnimSequenceBase::PostLoad_PopulatingModelInterface","Populating Animation Data Model Interface"), bDoNotTransactAction);
			    
		    	bPopulatingDataModel = true;
		    	
		    	Controller->InitializeModel();
    
		    	PRAGMA_DISABLE_DEPRECATION_WARNINGS
				// In case a data model has already been created populate the new one with its data
				if (DataModel != nullptr)
				{
					DataModel->ConditionalPreload();
					DataModel->PostLoad();
					DataModel->ConditionalPostLoadSubobjects();
					PopulateWithExistingModel(DataModel.Get());

					TScriptInterface<IAnimationDataController> DataModelController = DataModel->GetController();
					DataModelController->ResetModel(false);
					
					DataModel = nullptr;
					Controller->NotifyPopulated();
				}
		    	PRAGMA_ENABLE_DEPRECATION_WARNINGS
				// If switching to a different DataModelInterface implementation, copy the data from the existing one
				else if (CachedDataModelInterface != nullptr)
				{
					PopulateWithExistingModel(CachedDataModelInterface);
					Controller->NotifyPopulated();
				}
		    	// Otherwise upgrade this animation asset to be model-based
		    	else
		    	{
					PopulateModel();
		    		Controller->NotifyPopulated();
		    	}
		    	bPopulatingDataModel = false;
		    	Controller->CloseBracket(bDoNotTransactAction);
		    }
		    else
		    {
		    	// Fix-up to ensure correct curves are used for compression
		    	PRAGMA_DISABLE_DEPRECATION_WARNINGS
				RawCurveData.FloatCurves = DataModelInterface->GetCurveData().FloatCurves;
		    	RawCurveData.TransformCurves = DataModelInterface->GetCurveData().TransformCurves;
		    	RawCurveData.RemoveRedundantKeys(0.f);
		    	PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
		else
		{
			PreloadSkeleton();
		}
	}
#endif // WITH_EDITORONLY_DATA

	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Convert Notifies to new data
	if( GIsEditor && Notifies.Num() > 0 )
	{
		if(GetLinkerUEVersion() < VER_UE4_CLEAR_NOTIFY_TRIGGERS)
		{
			for(FAnimNotifyEvent Notify : Notifies)
			{
				if(Notify.Notify)
				{
					// Clear end triggers for notifies that are not notify states
					Notify.EndTriggerTimeOffset = 0.0f;
				}
			}
		}
	}

	InitializeNotifyTrack();
#endif	// WITH_EDITORONLY_DATA
	RefreshCacheData();

#if WITH_EDITORONLY_DATA
	if (!GetPackage()->GetHasBeenEndLoaded())
	{
		FCoreUObjectDelegates::OnEndLoadPackage.AddUObject(this, &UAnimSequenceBase::OnEndLoadPackage);
	}
	else
	{
		OnAnimModelLoaded();
	}

	if(USkeleton* MySkeleton = GetSkeleton())
	{
		if (IsDataModelValid())
		{
			const bool bDoNotTransactAction = false;
			if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FixUpNoneNameAnimationCurves)
			{
				Controller->OpenBracket(LOCTEXT("FFortniteMainBranchObjectVersion::FixUpNoneNameAnimationCurves_Bracket","FFortniteMainBranchObjectVersion::FixUpNoneNameAnimationCurves"), bDoNotTransactAction);
				{
					const TArray<FFloatCurve>& FloatCurves = DataModelInterface->GetFloatCurves();
					for (int32 Index = 0; Index < FloatCurves.Num(); ++Index)
					{
						const FFloatCurve& Curve = FloatCurves[Index];
						if (Curve.GetName() == NAME_None)
						{
							// give unique name
							const FName UniqueName = FName(*FString(GetName() + TEXT("_CurveNameFix_") + FString::FromInt(Index)));
							UE_LOGF(LogAnimation, Warning, "[AnimSequence %ls] contains invalid curve name \'None\'. Renaming this to %ls. Please fix this curve in the editor. ", *GetFullName(), *Curve.GetName().ToString());

							Controller->RenameCurve(FAnimationCurveIdentifier(Curve.GetName(), ERawCurveTrackTypes::RCT_Float), FAnimationCurveIdentifier(UniqueName, ERawCurveTrackTypes::RCT_Float), bDoNotTransactAction);
						}
					}
				}
				Controller->CloseBracket(bDoNotTransactAction);
			}
		}

		// this should continue to add if skeleton hasn't been saved either 
		// we don't wipe out data, so make sure you add back in if required
		if (GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::MoveCurveTypesToSkeleton
			|| MySkeleton->GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::MoveCurveTypesToSkeleton)
		{
			// This is safe as the data model will have been created during this PostLoad call
			const TArray<FFloatCurve>& FloatCurves = DataModelInterface->GetFloatCurves();

			// fix up curve flags to skeleton
			for (const FFloatCurve& Curve : FloatCurves)
			{
				bool bMorphtargetSet = Curve.GetCurveTypeFlag(AACF_DriveMorphTarget_DEPRECATED);
				bool bMaterialSet = Curve.GetCurveTypeFlag(AACF_DriveMaterial_DEPRECATED);

				// only add this if that has to 
				if (bMorphtargetSet || bMaterialSet)
				{
					MySkeleton->AccumulateCurveMetaData(Curve.GetName(), bMaterialSet, bMorphtargetSet);
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UAnimSequenceBase::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

#if WITH_EDITOR
	if (!GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		checkf(DataModelInterface.GetObject()->GetOuter() == this, TEXT("Animation Data Model interface has incorrect outer, expected %s - found %s"), *this->GetName(), *DataModelInterface.GetObject()->GetOuter()->GetName());
		BindToModelModificationEvent();
	}
#endif // WITH_EDITOR
}

float UAnimSequenceBase::GetPlayLength() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return SequenceLength;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAnimSequenceBase::SortNotifies()
{
	// Sorts using FAnimNotifyEvent::operator<()
	Notifies.Sort();
}

bool UAnimSequenceBase::RemoveNotifies(const TArray<FName>& NotifiesToRemove)
{
	bool bSequenceModified = false;
	for (int32 NotifyIndex = Notifies.Num() - 1; NotifyIndex >= 0; --NotifyIndex)
	{
		FAnimNotifyEvent& AnimNotify = Notifies[NotifyIndex];
		if (NotifiesToRemove.Contains(AnimNotify.NotifyName))
		{
			if (!bSequenceModified)
			{
				Modify();
				bSequenceModified = true;
			}
			Notifies.RemoveAtSwap(NotifyIndex);
		}
	}

	if (bSequenceModified)
	{
		MarkPackageDirty();
		RefreshCacheData();
	}
	return bSequenceModified;
}

void UAnimSequenceBase::RemoveNotifies()
{
	if (Notifies.Num() == 0)
	{
		return;
	}
	Modify();
	Notifies.Reset();
	MarkPackageDirty();
	RefreshCacheData();
}

#if WITH_EDITOR
void UAnimSequenceBase::RenameNotifies(FName InOldName, FName InNewName)
{
	Modify();

	for(FAnimNotifyEvent& Notify : Notifies)
	{
		// Only handle named notifies
		if(!Notify.IsBlueprintNotify())
		{
			if(Notify.NotifyName == InOldName)
			{
				Notify.NotifyName = InNewName;
			}
		}
	}

	// notification broadcast
	OnNotifyChanged.Broadcast();
}
#endif

bool UAnimSequenceBase::IsNotifyAvailable() const
{
	return (Notifies.Num() != 0) && (GetPlayLength() > 0.f);
}

void UAnimSequenceBase::GetAnimNotifies(const float& StartTime, const float& DeltaTime, FAnimNotifyContext& NotifyContext) const
{
	// Early out if we have no notifies
	if (!IsNotifyAvailable())
	{
		return;
	}
	
	bool const bPlayingBackwards = (DeltaTime < 0.f);
	float PreviousPosition = StartTime;
	float CurrentPosition = StartTime;
	float DesiredDeltaMove = DeltaTime;
	const float PlayLength = GetPlayLength();

	// previous behaviour could get the same notify multiple times  - support this within reasonable limits
	uint32_t MaxLoopCount = 2;
	if (PlayLength > 0.0f && FMath::Abs(DeltaTime) > PlayLength)
	{
		MaxLoopCount = FMath::Clamp(uint32_t(DesiredDeltaMove / PlayLength), 2, 1000);
	}

	for (uint32_t i = 0; i < MaxLoopCount; i++)
	{
		// Disable looping here. Advance to desired position, or beginning / end of animation
		const ETypeAdvanceAnim AdvanceType = FAnimationRuntime::AdvanceTime(false, DesiredDeltaMove, CurrentPosition, PlayLength);

		// Verify position assumptions
		ensureMsgf(bPlayingBackwards ? (CurrentPosition <= PreviousPosition) : (CurrentPosition >= PreviousPosition), TEXT("in Animation %s(Skeleton %s) : bPlayingBackwards(%d), PreviousPosition(%0.2f), Current Position(%0.2f)"),
			*GetName(), *GetNameSafe(GetSkeleton()), bPlayingBackwards, PreviousPosition, CurrentPosition);

		GetAnimNotifiesFromDeltaPositions(PreviousPosition, CurrentPosition, NotifyContext);

		// If we've hit the end of the animation, and we're allowed to loop, keep going.
		if ((AdvanceType == ETAA_Finished) && NotifyContext.TickRecord && NotifyContext.TickRecord->bLooping)
		{
			const float ActualDeltaMove = (CurrentPosition - PreviousPosition);
			DesiredDeltaMove -= ActualDeltaMove;
			PreviousPosition = bPlayingBackwards ? GetPlayLength() : 0.f;
			CurrentPosition = PreviousPosition;
		}
		else
		{
			break;
		}
	}
}

void UAnimSequenceBase::GetAnimNotifiesFromDeltaPositions(const float& PreviousPosition, const float& CurrentPosition,  FAnimNotifyContext& NotifyContext) const
{
	// Early out if we have no notifies
	if (Notifies.Num() == 0)
	{
		return;
	}

	bool const bPlayingBackwards = (CurrentPosition < PreviousPosition);

	// If playing backwards, flip Min and Max.
	if( bPlayingBackwards )
	{
		for (int32 NotifyIndex=0; NotifyIndex<Notifies.Num(); NotifyIndex++)
		{
			const FAnimNotifyEvent& AnimNotifyEvent = Notifies[NotifyIndex];
			const float NotifyStartTime = AnimNotifyEvent.GetTriggerTime();
			const float NotifyEndTime = AnimNotifyEvent.GetEndTriggerTime();

			if( (NotifyStartTime < PreviousPosition) && (NotifyEndTime >= CurrentPosition) )
			{
				if (NotifyContext.TickRecord)
				{
					NotifyContext.ActiveNotifies.Emplace(&AnimNotifyEvent, this, NotifyContext.TickRecord->MirrorDataTable);
					NotifyContext.ActiveNotifies.Top().GatherTickRecordData(*NotifyContext.TickRecord);
				}
				else
				{
					NotifyContext.ActiveNotifies.Emplace(&AnimNotifyEvent, this, nullptr);
				}

				const bool bHasFinished = CurrentPosition <= FMathf::Max(NotifyStartTime, 0.f);
				if (bHasFinished)
				{
					NotifyContext.ActiveNotifies.Top().AddContextData<UE::Anim::FAnimNotifyEndDataContext>(true);
				}
			}
		}
	}
	else
	{
		for (int32 NotifyIndex=0; NotifyIndex<Notifies.Num(); NotifyIndex++)
		{
			const FAnimNotifyEvent& AnimNotifyEvent = Notifies[NotifyIndex];
			const float NotifyStartTime = AnimNotifyEvent.GetTriggerTime();
			const float NotifyEndTime = AnimNotifyEvent.GetEndTriggerTime();

			// Note that if you arrive with zero delta time (CurrentPosition == PreviousPosition), only Notify States will be extracted
			if( (NotifyStartTime <= CurrentPosition) && (NotifyEndTime > PreviousPosition) )
			{
				if (NotifyContext.TickRecord)
				{
					NotifyContext.ActiveNotifies.Emplace(&AnimNotifyEvent, this, NotifyContext.TickRecord->MirrorDataTable);
					NotifyContext.ActiveNotifies.Top().GatherTickRecordData(*NotifyContext.TickRecord); 
				}
				else
				{
					NotifyContext.ActiveNotifies.Emplace(&AnimNotifyEvent, this, nullptr);
				}

				const bool bHasFinished = CurrentPosition >= NotifyEndTime;
				if (bHasFinished)
				{
					NotifyContext.ActiveNotifies.Top().AddContextData<UE::Anim::FAnimNotifyEndDataContext>(true);
				}
			}
		}
	}
}

void UAnimSequenceBase::TickAssetPlayer(FAnimTickRecord& Instance, struct FAnimNotifyQueue& NotifyQueue, FAnimAssetTickContext& Context) const
{
	// Local copy of the asset player's time that is about to be updated.
	float CurrentTime = *(Instance.TimeAccumulator);
	
	if (Context.ShouldResyncToSyncGroup() && !Instance.bIsEvaluator)
	{
		// Synchronize the asset player time to the other sync group members when (re)joining the group
		CurrentTime = Context.GetAnimationPositionRatio() * GetPlayLength();
	}

	float PreviousTime = CurrentTime;
	float DeltaTime = 0.f;
	const float PlayRate = Instance.PlayRateMultiplier * RateScale;

	if (Context.IsLeader())
	{
		DeltaTime = PlayRate * Context.GetDeltaTime();

		// Prepare context before ticking.
		Context.SetLeaderDelta(DeltaTime);
		Context.SetPreviousAnimationPositionRatio(PreviousTime / GetPlayLength());

		// Tick as leader using marked based syncing if possible, other fallback to normal ticking.
		if (DeltaTime != 0.f)
		{
			if (Instance.bCanUseMarkerSync && Context.CanUseMarkerPosition())
			{
				TickByMarkerAsLeader(*Instance.MarkerTickRecord, Context.MarkerTickContext, CurrentTime, PreviousTime, DeltaTime, Instance.bLooping, Instance.MirrorDataTable);
			}
			else
			{
				// Advance time
				FAnimationRuntime::AdvanceTime(Instance.bLooping, DeltaTime, CurrentTime, GetPlayLength());
				UE_LOGF(LogAnimMarkerSync, Log, "Leader (%ls) (normal advance)  - PreviousTime (%0.2f), CurrentTime (%0.2f), MoveDelta (%0.2f), Looping (%d) ", *GetName(), PreviousTime, CurrentTime, DeltaTime, Instance.bLooping ? 1 : 0);
			}
		}
		else if (Instance.bCanUseMarkerSync && Context.CanUseMarkerPosition() && !Instance.MarkerTickRecord->IsValid(Instance.bLooping))
		{
			// Re-compute marker indices since the asset's tick record is invalid. Get previous and next markers.
			GetMarkerIndicesForTime(CurrentTime, Instance.bLooping, Context.MarkerTickContext.GetValidMarkerNames(), Instance.MarkerTickRecord->PreviousMarker, Instance.MarkerTickRecord->NextMarker);
		}

		// Update context's data after ticking.
		Context.SetAnimationPositionRatio(CurrentTime / GetPlayLength());
	}
	else
	{
		// Follow the leader using marker based syncing if possible, otherwise fallback to length based syncing.
		if (Instance.bCanUseMarkerSync)
		{
			if (Context.CanUseMarkerPosition() && Context.MarkerTickContext.IsMarkerSyncStartValid())
			{
				TickByMarkerAsFollower(*Instance.MarkerTickRecord, Context.MarkerTickContext, CurrentTime, PreviousTime, Context.GetLeaderDelta(), Instance.bLooping, Instance.MirrorDataTable);
			}
			else
			{
				DeltaTime = PlayRate * Context.GetDeltaTime();
				
				// If leader is not valid, advance time as normal, do not jump position and pop.
				FAnimationRuntime::AdvanceTime(Instance.bLooping, DeltaTime, CurrentTime, GetPlayLength());
				UE_LOGF(LogAnimMarkerSync, Log, "Follower (%ls) (normal advance) - PreviousTime (%0.2f), CurrentTime (%0.2f), MoveDelta (%0.2f), Looping (%d) ", *GetName(), PreviousTime, CurrentTime, DeltaTime, Instance.bLooping ? 1 : 0);
			}
		}
		else
		{
			// Match group leader's last calculated anim position since we're not using marker based sync.
			PreviousTime = Context.GetPreviousAnimationPositionRatio() * GetPlayLength();
			CurrentTime = Context.GetAnimationPositionRatio() * GetPlayLength();
			
			UE_LOGF(LogAnimMarkerSync, Log, "Follower (%ls) (normalized position advance) - PreviousTime (%0.2f), CurrentTime (%0.2f), MoveDelta (%0.2f), Looping (%d) ", *GetName(), PreviousTime, CurrentTime, DeltaTime, Instance.bLooping ? 1 : 0);
		}
		
		if (CurrentTime != PreviousTime)
		{
			// Figure out delta time 
			DeltaTime = CurrentTime - PreviousTime;
			
			// If we went against play rate, then loop around.
			if ((DeltaTime * PlayRate) < 0.f)
			{
				DeltaTime += FMath::Sign<float>(PlayRate) * GetPlayLength();
			}
		}
	}

	// Update the instance's TimeAccumulator after all side effects on the local copy of the asset player's time have been applied.
	*(Instance.TimeAccumulator) = CurrentTime;
	
	// Capture the final adjusted delta time and previous frame time as an asset player record
	check(Instance.DeltaTimeRecord);
	Instance.DeltaTimeRecord->Set(PreviousTime, DeltaTime);

	// Allow asset to react right after its asset player has being ticked.
	HandleAssetPlayerTickedInternal(Context, PreviousTime, DeltaTime, Instance, NotifyQueue);
}

void UAnimSequenceBase::TickByMarkerAsFollower(FMarkerTickRecord &Instance, FMarkerTickContext &MarkerContext, float& CurrentTime, float& OutPreviousTime, const float MoveDelta, const bool bLooping, const UMirrorDataTable* MirrorTable) const
{
	// Re-compute marker indices since the asset's tick record is invalid. Get previous and next markers.
	if (!Instance.IsValid(bLooping))
	{
		GetMarkerIndicesForPosition(MarkerContext.GetMarkerSyncStartPosition(), bLooping, Instance.PreviousMarker, Instance.NextMarker, CurrentTime, MirrorTable);
	}

	// Keep track the asset's previous time, since its current time is about to be modified.
	OutPreviousTime = CurrentTime;

	// Tick and update as follower.
	AdvanceMarkerPhaseAsFollower(MarkerContext, MoveDelta, bLooping, CurrentTime, Instance.PreviousMarker, Instance.NextMarker, MirrorTable);

	UE_LOGF(LogAnimMarkerSync, Log, "Follower (%ls) (TickByMarker) PreviousTime(%0.2f) CurrentTime(%0.2f) MoveDelta(%0.2f) Looping(%d) %ls", *GetName(), OutPreviousTime, CurrentTime, MoveDelta, bLooping ? 1 : 0, *MarkerContext.ToString());
}

void UAnimSequenceBase::TickByMarkerAsLeader(FMarkerTickRecord& Instance, FMarkerTickContext& MarkerContext, float& CurrentTime, float& OutPreviousTime, const float MoveDelta, const bool bLooping, const UMirrorDataTable* MirrorTable) const
{
	// Re-compute marker indices since the asset's tick record is invalid. Get previous and next markers.
	if (!Instance.IsValid(bLooping))
	{
		if (MarkerContext.IsMarkerSyncStartValid())
		{
			GetMarkerIndicesForPosition(MarkerContext.GetMarkerSyncStartPosition(), bLooping, Instance.PreviousMarker, Instance.NextMarker, CurrentTime, MirrorTable);
		}
		else
		{
			GetMarkerIndicesForTime(CurrentTime, bLooping, MarkerContext.GetValidMarkerNames(), Instance.PreviousMarker, Instance.NextMarker);
		}
	}

	// Store the sync anim position BEFORE the asset has being ticked.
	MarkerContext.SetMarkerSyncStartPosition(GetMarkerSyncPositionFromMarkerIndicies(Instance.PreviousMarker.MarkerIndex, Instance.NextMarker.MarkerIndex, CurrentTime, MirrorTable));

	// Keep track the asset's previous time, since its current time is about to be modified.
	OutPreviousTime = CurrentTime;

	// Advance as leader.
	AdvanceMarkerPhaseAsLeader(bLooping, MoveDelta, MarkerContext.GetValidMarkerNames(), CurrentTime, Instance.PreviousMarker, Instance.NextMarker, MarkerContext.MarkersPassedThisTick, MirrorTable);

	// Store the sync anim position AFTER the asset has being ticked.
	MarkerContext.SetMarkerSyncEndPosition(GetMarkerSyncPositionFromMarkerIndicies(Instance.PreviousMarker.MarkerIndex, Instance.NextMarker.MarkerIndex, CurrentTime, MirrorTable));

	UE_LOGF(LogAnimMarkerSync, Log, "Leader (%ls) (TickByMarker) PreviousTime(%0.2f) CurrentTime(%0.2f) MoveDelta(%0.2f) Looping(%d) %ls", *GetName(), OutPreviousTime, CurrentTime, MoveDelta, bLooping ? 1 : 0, *MarkerContext.ToString());
}

bool CanNotifyUseTrack(const FAnimNotifyTrack& Track, const FAnimNotifyEvent& Notify)
{
	for (const FAnimNotifyEvent* Event : Track.Notifies)
	{
		if (FMath::IsNearlyEqual(Event->GetTime(), Notify.GetTime()))
		{
			return false;
		}
	}
	return true;
}

FAnimNotifyTrack& AddNewTrack(TArray<FAnimNotifyTrack>& Tracks)
{
	const int32 Index = Tracks.Add(FAnimNotifyTrack(*FString::FromInt(Tracks.Num() + 1), FLinearColor::White));
	return Tracks[Index];
}

void UAnimSequenceBase::RefreshCacheData()
{
	SortNotifies();

#if WITH_EDITOR
	for (int32 TrackIndex = 0; TrackIndex < AnimNotifyTracks.Num(); ++TrackIndex)
	{
		AnimNotifyTracks[TrackIndex].Notifies.Empty();
	}

	for (FAnimNotifyEvent& Notify : Notifies)
	{
		// Handle busted track indices
		if (!AnimNotifyTracks.IsValidIndex(Notify.TrackIndex))
		{
			// This really shouldn't happen (unless we are a cooked asset), but try to handle it
			ensureMsgf(GetOutermost()->bIsCookedForEditor, TEXT("AnimNotifyTrack: Anim (%s) has notify (%s) with track index (%i) that does not exist"), *GetFullName(), *Notify.NotifyName.ToString(), Notify.TrackIndex);

			// Don't create lots of extra tracks if we are way off supporting this track
			if (Notify.TrackIndex < 0 || Notify.TrackIndex > 20)
			{
				Notify.TrackIndex = 0;
			}

			while (!AnimNotifyTracks.IsValidIndex(Notify.TrackIndex))
			{
				AddNewTrack(AnimNotifyTracks);
			}
		}

		// Handle overlapping notifies
		FAnimNotifyTrack* TrackToUse = nullptr;
		int32 TrackIndexToUse = INDEX_NONE;
		for (int32 TrackOffset = 0; TrackOffset < AnimNotifyTracks.Num(); ++TrackOffset)
		{
			const int32 TrackIndex = (Notify.TrackIndex + TrackOffset) % AnimNotifyTracks.Num();
			if (CanNotifyUseTrack(AnimNotifyTracks[TrackIndex], Notify))
			{
				TrackToUse = &AnimNotifyTracks[TrackIndex];
				TrackIndexToUse = TrackIndex;
				break;
			}
		}

		if (TrackToUse == nullptr)
		{
			TrackToUse = &AddNewTrack(AnimNotifyTracks);
			TrackIndexToUse = AnimNotifyTracks.Num() - 1;
		}

		check(TrackToUse);
		check(TrackIndexToUse != INDEX_NONE);

		Notify.TrackIndex = TrackIndexToUse;
		TrackToUse->Notifies.Add(&Notify);
	}

	// this is a separate loop of checkin if they contains valid notifies
	for (int32 NotifyIndex = 0; NotifyIndex < Notifies.Num(); ++NotifyIndex)
	{
		const FAnimNotifyEvent& Notify = Notifies[NotifyIndex];
		// make sure if they can be placed in editor
		if (Notify.Notify)
		{
			if (Notify.Notify->CanBePlaced(this) == false)
			{
				static FName NAME_LoadErrors("LoadErrors");
				FMessageLog LoadErrors(NAME_LoadErrors);

				TSharedRef<FTokenizedMessage> Message = LoadErrors.Error();
				Message->AddToken(FTextToken::Create(LOCTEXT("InvalidAnimNotify1", "The Animation ")));
				Message->AddToken(FAssetNameToken::Create(GetPathName(), FText::FromString(GetNameSafe(this))));
				Message->AddToken(FTextToken::Create(LOCTEXT("InvalidAnimNotify2", " contains invalid notify ")));
				Message->AddToken(FAssetNameToken::Create(Notify.Notify->GetPathName(), FText::FromString(GetNameSafe(Notify.Notify))));
				LoadErrors.Open();
			}
		}

		if (Notify.NotifyStateClass)
		{
			if (Notify.NotifyStateClass->CanBePlaced(this) == false)
			{
				static FName NAME_LoadErrors("LoadErrors");
				FMessageLog LoadErrors(NAME_LoadErrors);

				TSharedRef<FTokenizedMessage> Message = LoadErrors.Error();
				Message->AddToken(FTextToken::Create(LOCTEXT("InvalidAnimNotify1", "The Animation ")));
				Message->AddToken(FAssetNameToken::Create(GetPathName(), FText::FromString(GetNameSafe(this))));
				Message->AddToken(FTextToken::Create(LOCTEXT("InvalidAnimNotify2", " contains invalid notify ")));
				Message->AddToken(FAssetNameToken::Create(Notify.NotifyStateClass->GetPathName(), FText::FromString(GetNameSafe(Notify.NotifyStateClass))));
				LoadErrors.Open();
			}
		}
	}
	// notification broadcast
	OnNotifyChanged.Broadcast();
#endif //WITH_EDITOR
}

int32 UAnimSequenceBase::GetNumberOfSampledKeys() const
{
	return GetSamplingFrameRate().AsFrameTime(GetPlayLength()).RoundToFrame().Value;
}

FFrameRate UAnimSequenceBase::GetSamplingFrameRate() const
{
	static const FFrameRate DefaultFrameRate = UAnimationSettings::Get()->GetDefaultFrameRate();
	return DefaultFrameRate;
}

float UAnimSequenceBase::GetTimeAtFrame(const int32 Frame) const
{
	return FMath::Clamp((float)GetSamplingFrameRate().AsSeconds(Frame), 0.f, GetPlayLength());
}

#if WITH_EDITOR
void UAnimSequenceBase::InitializeNotifyTrack()
{
	if ( AnimNotifyTracks.Num() == 0 ) 
	{
		AnimNotifyTracks.Add(FAnimNotifyTrack(TEXT("1"), FLinearColor::White ));
	}
}

int32 UAnimSequenceBase::GetFrameAtTime(const float Time) const
{
	return FMath::Clamp(GetSamplingFrameRate().AsFrameTime(Time).RoundToFrame().Value, 0, GetNumberOfSampledKeys() - 1);
}

void UAnimSequenceBase::RegisterOnNotifyChanged(const FOnNotifyChanged& Delegate)
{
	OnNotifyChanged.Add(Delegate);
}
void UAnimSequenceBase::UnregisterOnNotifyChanged(FDelegateUserObject Unregister)
{
	OnNotifyChanged.RemoveAll(Unregister);
}

void UAnimSequenceBase::ClampNotifiesAtEndOfSequence()
{
	const float NotifyClampTime = GetPlayLength();
	for(int i = 0; i < Notifies.Num(); ++ i)
	{
		if(Notifies[i].GetTime() >= GetPlayLength())
		{
			Notifies[i].SetTime(NotifyClampTime);
			Notifies[i].TriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::OffsetBefore);
		}
	}
}

EAnimEventTriggerOffsets::Type UAnimSequenceBase::CalculateOffsetForNotify(float NotifyDisplayTime) const
{
	if(NotifyDisplayTime == 0.f)
	{
		return EAnimEventTriggerOffsets::OffsetAfter;
	}
	else if(NotifyDisplayTime == GetPlayLength())
	{
		return EAnimEventTriggerOffsets::OffsetBefore;
	}
	return EAnimEventTriggerOffsets::NoOffset;
}

void UAnimSequenceBase::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	// Add notify IDs to a tag list, or a delimiter if we have no notifies.
	// The delimiter is necessary so we can distinguish between data with no curves and old data, as the asset registry
	// strips tags that have empty values 
	FString NotifyList = USkeleton::AnimNotifyTagDelimiter;
	for(auto Iter=Notifies.CreateConstIterator(); Iter; ++Iter)
	{
		// only add if not BP anim notify since they're handled separately
		if(Iter->IsBlueprintNotify() == false)
		{
			NotifyList += FString::Printf(TEXT("%s%s"), *Iter->NotifyName.ToString(), *USkeleton::AnimNotifyTagDelimiter);
		}
	}
	
	Context.AddTag(FAssetRegistryTag(USkeleton::AnimNotifyTag, NotifyList, FAssetRegistryTag::TT_Hidden));

	// Add curve IDs to a tag list, or a delimiter if we have no curves.
	// The delimiter is necessary so we can distinguish between data with no curves and old data, as the asset registry
	// strips tags that have empty values 
	FString CurveNameList = USkeleton::CurveTagDelimiter;
	if (DataModelInterface.GetObject())
	{
		for(const FFloatCurve& Curve : DataModelInterface->GetFloatCurves())
		{
			CurveNameList += FString::Printf(TEXT("%s%s"), *Curve.GetName().ToString(), *USkeleton::CurveTagDelimiter);
		}
	}
	Context.AddTag(FAssetRegistryTag(USkeleton::CurveNameTag, CurveNameList, FAssetRegistryTag::TT_Hidden));

	if (DataModelInterface.GetObject())
	{
		FAnimationAttributeIdentifierArray AttributeIdentifierArray;
		AttributeIdentifierArray.AttributeIdentifiers.Reserve(DataModelInterface->GetNumberOfAttributes());
		
		for (const FAnimatedBoneAttribute& BoneAttribute : DataModelInterface->GetAttributes())
		{
			AttributeIdentifierArray.AttributeIdentifiers.Add(BoneAttribute.Identifier);
		}

		FString AttributeList;
		FAnimationAttributeIdentifierArray::StaticStruct()->ExportText(AttributeList, &AttributeIdentifierArray.AttributeIdentifiers, nullptr, nullptr, PPF_None, nullptr);
		
		Context.AddTag(FAssetRegistryTag(USkeleton::AttributeTag, AttributeList, FAssetRegistryTag::TT_Hidden));
	}
}

uint8* UAnimSequenceBase::FindNotifyPropertyData(int32 NotifyIndex, FArrayProperty*& ArrayProperty)
{
	// initialize to nullptr
	ArrayProperty = nullptr;

	if(Notifies.IsValidIndex(NotifyIndex))
	{
		return FindArrayProperty(TEXT("Notifies"), ArrayProperty, NotifyIndex);
	}
	return nullptr;
}

uint8* UAnimSequenceBase::FindArrayProperty(const TCHAR* PropName, FArrayProperty*& ArrayProperty, int32 ArrayIndex)
{
	// find Notifies property start point
	FProperty* Property = FindFProperty<FProperty>(GetClass(), PropName);

	// found it and if it is array
	if (Property && Property->IsA(FArrayProperty::StaticClass()))
	{
		// find Property Value from UObject we got
		uint8* PropertyValue = Property->ContainerPtrToValuePtr<uint8>(this);

		// it is array, so now get ArrayHelper and find the raw ptr of the data
		ArrayProperty = CastFieldChecked<FArrayProperty>(Property);
		FScriptArrayHelper ArrayHelper(ArrayProperty, PropertyValue);

		if (ArrayProperty->Inner && ArrayIndex < ArrayHelper.Num())
		{
			//Get property data based on selected index
			return ArrayHelper.GetRawPtr(ArrayIndex);
		}
	}
	return nullptr;
}

void UAnimSequenceBase::RefreshParentAssetData()
{
	Super::RefreshParentAssetData();

	check(ParentAsset);

	UAnimSequenceBase* ParentSeqBase = CastChecked<UAnimSequenceBase>(ParentAsset);

	RateScale = ParentSeqBase->RateScale;

	ValidateModel();
	Controller->OpenBracket(LOCTEXT("RefreshParentAssetData_Bracket", "Refreshing Parent Asset Data"));
	{	
		const IAnimationDataModel* ParentDataModel = ParentSeqBase->GetDataModel();
		
		if (!FMath::IsNearlyEqual(GetPlayLength(), ParentSeqBase->GetPlayLength()))
		{
			const FFrameRate& ParentFrameRate = ParentDataModel->GetFrameRate();
			const FFrameRate& ChildFrameRate = GetDataModelInterface()->GetFrameRate();

			if (ChildFrameRate.IsMultipleOf(ParentFrameRate) || ChildFrameRate.IsFactorOf(ParentFrameRate))
			{
				// Ensure that play lengths are the same by converting the parent number of frames into the child's frame rate
				FFrameTime ChildNumFrames = FFrameRate::TransformTime(ParentFrameRate.AsFrameTime(ParentSeqBase->GetPlayLength()), ParentFrameRate, ChildFrameRate);
				Controller->SetNumberOfFrames(ChildNumFrames.RoundToFrame());
			}
			else
			{
				UE_LOGF(LogTemp, Display,
						"Child %ls %ls has incompatible frame rate with Parent %ls. Child: %ls, Parent: %ls. Reimport the child animation with a compatible frame rate.",
					*ParentSeqBase->GetClass()->GetName(), 
					*GetName(), 
					*ParentSeqBase->GetName(), 
					*ChildFrameRate.ToPrettyText().ToString(), 
					*ParentFrameRate.ToPrettyText().ToString());
			}
		}
		
		Controller->RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float);
		for (const FFloatCurve& FloatCurve : ParentDataModel->GetFloatCurves())
		{
			const FAnimationCurveIdentifier CurveId(FloatCurve.GetName(), ERawCurveTrackTypes::RCT_Float);
			Controller->AddCurve(CurveId, FloatCurve.GetCurveTypeFlags());
			Controller->SetCurveKeys(CurveId, FloatCurve.FloatCurve.GetConstRefOfKeys());
		}

		Controller->RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform);
		for (const FTransformCurve& TransformCurve : ParentDataModel->GetTransformCurves())
		{
			const FAnimationCurveIdentifier CurveId(TransformCurve.GetName(), ERawCurveTrackTypes::RCT_Transform);
			Controller->AddCurve(CurveId, TransformCurve.GetCurveTypeFlags());

			// Set each individual channel rich curve keys, to account for any custom tangents etc.
			for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
			{
				const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
				const FVectorCurve* VectorCurve = TransformCurve.GetVectorCurveByIndex(SubCurveIndex);
				for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
				{
					const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
					FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
					UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
					Controller->SetCurveKeys(TargetCurveIdentifier, VectorCurve->FloatCurves[ChannelIndex].GetConstRefOfKeys());
				}
			}
		}
	}
	Controller->CloseBracket();

	// should do deep copy because notify contains outer
	Notifies = ParentSeqBase->Notifies;
	// update notify
	for (int32 NotifyIdx = 0; NotifyIdx < Notifies.Num(); ++NotifyIdx)
	{
		FAnimNotifyEvent& NotifyEvent = Notifies[NotifyIdx];
		if (NotifyEvent.Notify)
		{
			class UAnimNotify* NewNotifyClass = DuplicateObject(NotifyEvent.Notify, this);
			NotifyEvent.Notify = NewNotifyClass;
		}
		if (NotifyEvent.NotifyStateClass)
		{
			class UAnimNotifyState* NewNotifyStateClass = DuplicateObject(NotifyEvent.NotifyStateClass, this);
			NotifyEvent.NotifyStateClass = (NewNotifyStateClass);
		}

		NotifyEvent.Link(this, NotifyEvent.GetTime(), NotifyEvent.GetSlotIndex());
		NotifyEvent.EndLink.Link(this, NotifyEvent.GetTime() + NotifyEvent.Duration, NotifyEvent.GetSlotIndex());
	}
#if WITH_EDITORONLY_DATA
	// if you change Notifies array, this will need to be rebuilt
	AnimNotifyTracks = ParentSeqBase->AnimNotifyTracks;

	// fix up notify links, brute force and ugly code
	for (FAnimNotifyTrack& Track : AnimNotifyTracks)
	{
		for (FAnimNotifyEvent*& Notify : Track.Notifies)
		{
			for (int32 ParentNotifyIdx = 0; ParentNotifyIdx < ParentSeqBase->Notifies.Num(); ++ParentNotifyIdx)
			{
				if (Notify == &ParentSeqBase->Notifies[ParentNotifyIdx])
				{
					Notify = &Notifies[ParentNotifyIdx];
					break;
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

}
#endif	//WITH_EDITOR


void UAnimSequenceBase::EvaluateCurveData(FBlendedCurve& OutCurve, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData) const
{
	CSV_SCOPED_TIMING_STAT(Animation, EvaluateCurveData);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RawCurveData.EvaluateCurveData(OutCurve, AnimExtractContext.CurrentTime);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

float UAnimSequenceBase::EvaluateCurveData(FName CurveName, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FFloatCurve* Curve = (const FFloatCurve*)RawCurveData.GetCurveData(CurveName, ERawCurveTrackTypes::RCT_Float);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return Curve != nullptr ? Curve->Evaluate(AnimExtractContext.CurrentTime) : 0.0f;
}

const FRawCurveTracks& UAnimSequenceBase::GetCurveData() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RawCurveData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAnimSequenceBase::HasCurveData(FName CurveName, bool bForceUseRawData) const
{
#if WITH_EDITOR
	if (IsDataModelValid())
	{
		return DataModelInterface->FindFloatCurve(FAnimationCurveIdentifier(CurveName, ERawCurveTrackTypes::RCT_Float)) != nullptr;
	}	
#endif
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RawCurveData.GetCurveData(CurveName) != nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAnimSequenceBase::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Super::Serialize(Ar);

	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::AnimationDataModelInterface_BackedOut &&
		GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::BackoutAnimationDataModelInterface)
	{
		UE_LOGF(LogAnimation, Fatal, "This package (%ls) was saved with a version that had to be backed out and is no longer able to be loaded.", *GetPathNameSafe(this));
	}

	// fix up version issue and so on
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RawCurveData.PostSerializeFixup(Ar);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAnimSequenceBase::HandleAssetPlayerTickedInternal(FAnimAssetTickContext &Context, const float PreviousTime, const float MoveDelta, const FAnimTickRecord &Instance, struct FAnimNotifyQueue& NotifyQueue) const
{
	// Harvest and record notifies
	FAnimNotifyContext NotifyContext(Instance);
	GetAnimNotifies(PreviousTime, MoveDelta, NotifyContext);
	NotifyQueue.AddAnimNotifies(Context.ShouldGenerateNotifies(), NotifyContext.ActiveNotifies, Instance.EffectiveBlendWeight);
}

#if WITH_EDITOR
void UAnimSequenceBase::OnModelModified(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& Payload)
{
	NotifyCollector.Handle(NotifyType);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InitializeNotifyTrack();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	auto HandleNumberOfFramesChange = [&](int32 NewNumFrames, int32 OldNumFrames, int32 Frame0, int32 Frame1)
	{
		// Do not handle changes during model population, or undo-redo (notifies are transacted so will be restored/set, so only handle in case of, initial, user interaction)
		if (bPopulatingDataModel || GIsTransacting)
		{
			return;
		}

		const FFrameRate& ModelFrameRate = DataModelInterface->GetFrameRate();
		const float T0 = ModelFrameRate.AsSeconds(Frame0);
		const float T1 = ModelFrameRate.AsSeconds(Frame1);
		const float NewLength = ModelFrameRate.AsSeconds(NewNumFrames);
		
		const float StartTime = ModelFrameRate.AsSeconds(Frame0);
		const float EndTime = ModelFrameRate.AsSeconds(Frame1);

		// Total time value for frames that were removed
		const float Duration = T1 - T0;

		if (NewNumFrames > OldNumFrames)
		{
			const float InsertTime = T0;
			for (FAnimNotifyEvent& Notify : Notifies)
			{
				// Proportional notifies don't need to be adjusted
				if (Notify.GetLinkMethod() == EAnimLinkMethod::Proportional)
				{
					continue;
				}

				float CurrentTime = Notify.GetTime();
				float NewDuration = Notify.GetDuration();

				// if state, make sure to adjust timings
				if (Notify.NotifyStateClass)
				{
					const float NotifyDuration = Notify.GetDuration();
					const float NotifyEnd = CurrentTime + NotifyDuration;

					TRange<float> NotifyRange(CurrentTime, TRange<float>::BoundsType(CurrentTime+NotifyDuration));
					TRange<float> InsertionRange(StartTime, EndTime);

					// If insertion range or start time falls inside of notify range, add length to notify duration
					if (NotifyRange.Contains(InsertionRange) || NotifyRange.Contains(StartTime))
					{
						// Removal range falls entirely inside of notify state duration
						const float InsertDuration = InsertionRange.GetUpperBound().GetValue() - InsertionRange.GetLowerBound().GetValue();
						NewDuration += InsertDuration;
					}					
				}

				// Shift out notify by the time allotted for the inserted keys
				if (CurrentTime >= InsertTime)
				{
					CurrentTime += Duration;
				}

				// Clamps against the sequence length, ensuring that the notify does not end up outside of the playback bounds
				const float ClampedCurrentTime = FMath::Clamp(CurrentTime, 0.f, NewLength);

				Notify.Link(this, ClampedCurrentTime);
				Notify.SetDuration(NewDuration);

				if (ClampedCurrentTime == 0.f)
				{
					Notify.TriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::OffsetAfter);
				}
				else if (ClampedCurrentTime == NewLength)
				{
					Notify.TriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::OffsetBefore);
				}
			}
		}
		else if (NewNumFrames < OldNumFrames)
		{
			// re-locate notifies
			for (FAnimNotifyEvent& Notify : Notifies)
			{
				// Proportional notifies don't need to be adjusted
				if (Notify.GetLinkMethod() == EAnimLinkMethod::Proportional)
				{
					continue;
				}

				float CurrentTime = Notify.GetTime();
				float NewDuration = 0.f;
				
				// if state, make sure to adjust end time
				if (Notify.NotifyStateClass)
				{
					const float NotifyDuration = Notify.GetDuration();
					const float NotifyEnd = CurrentTime + NotifyDuration;
					NewDuration = NotifyDuration;

					TRange<float> NotifyRange(CurrentTime, TRange<float>::BoundsType(CurrentTime+NotifyDuration));					
					TRange<float> RemovalRange(StartTime, EndTime);

					// If Notify is inside of the trimmed time frame, zero out the duration
					if (RemovalRange.Contains(NotifyRange))
					{
						// small number @todo see if there is define for this
						NewDuration = DataModelInterface->GetFrameRate().AsInterval();
					}
					else if (NotifyRange.Contains(RemovalRange))
					{
						// Removal range falls entirely inside of notify state duration
						const float RemovalDuration = RemovalRange.GetUpperBound().GetValue() - RemovalRange.GetLowerBound().GetValue();
						NewDuration -= RemovalDuration;
					}
					// If Notify overlaps trimmed time frame, remove trimmed duration
					else if (RemovalRange.Overlaps(NotifyRange))
					{
						TRange<float> Overlap = TRange<float>::Intersection(NotifyRange, RemovalRange);
						if (Overlap.GetLowerBoundValue() > CurrentTime)
						{
							//Removing from back-end of notify
							NewDuration = Overlap.GetLowerBoundValue() - CurrentTime;
						}
						else if (Overlap.GetUpperBoundValue() < NotifyEnd)
						{
							// Removing from front-end of notify
							NewDuration = NotifyEnd - Overlap.GetUpperBoundValue();
						}
					}

					NewDuration = FMath::Max(NewDuration, (float)DataModelInterface->GetFrameRate().AsInterval());
				}

				if (CurrentTime >= StartTime && CurrentTime <= EndTime)
				{
					CurrentTime = StartTime;
				}
				else if (CurrentTime > EndTime)
				{
					CurrentTime -= Duration;
				}

				const float ClampedCurrentTime = FMath::Clamp(CurrentTime, 0.f, NewLength);

				Notify.Link(this, ClampedCurrentTime);
				Notify.SetDuration(NewDuration);

				if (ClampedCurrentTime == 0.f)
				{
					Notify.TriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::OffsetAfter);
				}
				else if (ClampedCurrentTime == NewLength)
				{
					Notify.TriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::OffsetBefore);
				}
			}
		}
	};

	// Copy any float/transform curves from the model into RawCurveData, as it is used at runtime for AnimMontage/Composite(s)
	auto CopyCurvesFromModel = [this]()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RawCurveData.FloatCurves = DataModelInterface->GetCurveData().FloatCurves;
		RawCurveData.TransformCurves = DataModelInterface->GetCurveData().TransformCurves;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	bool bShouldMarkPackageDirty = !FUObjectThreadContext::Get().IsRoutingPostLoad && NotifyType != EAnimDataModelNotifyType::BracketOpened;

	switch (NotifyType)
	{
		case EAnimDataModelNotifyType::SequenceLengthChanged:
		{
			const FSequenceLengthChangedPayload& TypedPayload = Payload.GetPayload<FSequenceLengthChangedPayload>();
			const FFrameNumber PreviousNumberOfFrames = TypedPayload.PreviousNumberOfFrames;
			const int32 CurrentNumberOfFrames = Model->GetNumberOfFrames();

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SequenceLength = Model->GetFrameRate().AsSeconds(CurrentNumberOfFrames);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			HandleNumberOfFramesChange(CurrentNumberOfFrames, PreviousNumberOfFrames.Value, TypedPayload.Frame0.Value, TypedPayload.Frame1.Value);

			// Ensure that we only clamp the notifies at the end of a bracket
			if (NotifyCollector.IsNotWithinBracket())
			{
				ClampNotifiesAtEndOfSequence();
				CopyCurvesFromModel();
			}
			
			break;
		}

		case EAnimDataModelNotifyType::CurveAdded:
		case EAnimDataModelNotifyType::CurveChanged:
		case EAnimDataModelNotifyType::CurveRemoved:
		case EAnimDataModelNotifyType::CurveFlagsChanged:
		case EAnimDataModelNotifyType::CurveScaled:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				CopyCurvesFromModel();
			}
			break;
		}
		
		case EAnimDataModelNotifyType::CurveRenamed:
		{
			const FCurveRenamedPayload& TypedPayload = Payload.GetPayload<FCurveRenamedPayload>();
			FAnimCurveBase* CurvePtr = [this, Identifier=TypedPayload.Identifier]() -> FAnimCurveBase*
			{
				if (Identifier.CurveType == ERawCurveTrackTypes::RCT_Float)
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return RawCurveData.FloatCurves.FindByPredicate([CurveName=Identifier.CurveName](FFloatCurve& Curve)
	                {
	                    return Curve.GetName() == CurveName;
	                });
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				else if (Identifier.CurveType == ERawCurveTrackTypes::RCT_Transform)
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return RawCurveData.TransformCurves.FindByPredicate([CurveName=Identifier.CurveName](FTransformCurve& Curve)
                    {
                        return Curve.GetName() == CurveName;
                    });
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}

				return nullptr;
			}();
				
			if (CurvePtr)
			{
				CurvePtr->SetName(TypedPayload.NewIdentifier.CurveName);
			}
			break;
		}

		case EAnimDataModelNotifyType::Populated:
		{
			const float CurrentSequenceLength = Model->GetPlayLength();
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SequenceLength = CurrentSequenceLength;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			if (NotifyCollector.IsNotWithinBracket())
			{
				ClampNotifiesAtEndOfSequence();
				CopyCurvesFromModel();
			}

			break;
		}

		case EAnimDataModelNotifyType::BracketClosed:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				const auto CurveCopyNotifies = { EAnimDataModelNotifyType::CurveAdded, EAnimDataModelNotifyType::CurveChanged, EAnimDataModelNotifyType::CurveRemoved, EAnimDataModelNotifyType::CurveFlagsChanged, EAnimDataModelNotifyType::CurveScaled, EAnimDataModelNotifyType::Populated, EAnimDataModelNotifyType::Reset };

				const auto LengthChangingNotifies = { EAnimDataModelNotifyType::SequenceLengthChanged, EAnimDataModelNotifyType::FrameRateChanged, EAnimDataModelNotifyType::Reset, EAnimDataModelNotifyType::Populated };

				bShouldMarkPackageDirty = NotifyCollector.WasDataModified();

				if (NotifyCollector.Contains(CurveCopyNotifies) || NotifyCollector.Contains(LengthChangingNotifies))
				{
					CopyCurvesFromModel();
				}
				
				if (NotifyCollector.Contains(LengthChangingNotifies))
				{
					CopyCurvesFromModel();
					ClampNotifiesAtEndOfSequence();
				}
			}
			break;
		}
	}

	if (NotifyCollector.IsNotWithinBracket())
	{
		if (bShouldMarkPackageDirty)
		{
			MarkPackageDirty();
		}
	}
	else if (bShouldMarkPackageDirty)
	{
		NotifyCollector.MarkDataModified();
	}
}

IAnimationDataModel* UAnimSequenceBase::GetDataModel() const
{
	return DataModelInterface.GetInterface();
}

TScriptInterface<IAnimationDataModel> UAnimSequenceBase::GetDataModelInterface() const
{
	return DataModelInterface;
}

IAnimationDataController& UAnimSequenceBase::GetController()
{
	ValidateModel();

	if(Controller == nullptr)
	{
		Controller = DataModelInterface->GetController();
		checkf(Controller, TEXT("Failed to create AnimationDataController"));
		Controller->SetModel(DataModelInterface);
	}

	ensureAlways(Controller->GetModelInterface() == DataModelInterface);

	return *Controller;
}

void UAnimSequenceBase::PopulateModel()
{
	check(!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject));	
	
	// Make a copy of the current data, to mitigate any changes due to notify callbacks
	const FFrameRate FrameRate = GetSamplingFrameRate();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const float PlayLength = SequenceLength;
	const FRawCurveTracks CurveData = RawCurveData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
	IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("UAnimSequenceBase::PopulateModel_Bracket", "Generating Animation Model Data for Animation Sequence Base"));
	Controller->SetFrameRate(FrameRate);
	Controller->SetNumberOfFrames(Controller->ConvertSecondsToFrameNumber(PlayLength));

	USkeleton* TargetSkeleton = GetSkeleton();	
	UE::Anim::CopyCurveDataToModel(CurveData, TargetSkeleton,  *Controller);	
}

void UAnimSequenceBase::PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> ExistingDataModel)
{
	Controller->PopulateWithExistingModel(ExistingDataModel);
}

void UAnimSequenceBase::BindToModelModificationEvent()
{
	ValidateModel();
	DataModelInterface->GetModifiedEvent().RemoveAll(this);
	DataModelInterface->GetModifiedEvent().AddUObject(this, &UAnimSequenceBase::OnModelModified);
}

void UAnimSequenceBase::CopyDataModel(const TScriptInterface<IAnimationDataModel>& ModelToDuplicate)
{
	checkf(ModelToDuplicate != nullptr, TEXT("Invalidate data model %s"), *GetFullName());
	if (ModelToDuplicate)
	{
		ModelToDuplicate->GetModifiedEvent().RemoveAll(this);
	}

	DataModelInterface = DuplicateObject(ModelToDuplicate.GetObject(), this);

	if(Controller)
	{
		Controller->SetModel(DataModelInterface);
	}
	else
	{
		GetController();
	}

	BindToModelModificationEvent();
}

void UAnimSequenceBase::CreateModel()
{
	const UClass* TargetClass = UE::Anim::DataModel::IAnimationDataModels::FindClassForAnimationAsset(this);
	checkf(TargetClass != nullptr, TEXT("Unable to find valid AnimationDataModel class"));

	checkf(!DataModelInterface || DataModelInterface.GetObject()->GetClass() != TargetClass, TEXT("Invalid attempt to override the existing data model %s"), *GetFullName());
	UObject* ClassDataModel = NewObject<UObject>(this, TargetClass, TargetClass->GetFName());
	DataModelInterface = ClassDataModel;

	BindToModelModificationEvent();
}

void UAnimSequenceBase::ValidateModel() const
{
	checkf(DataModelInterface != nullptr, TEXT("Invalid AnimSequenceBase state (%s), no data model found"), *GetPathName());
}

bool UAnimSequenceBase::ShouldDataModelBeValid() const
{
	return
#if WITH_EDITOR
		!GetOutermost()->HasAnyPackageFlags(PKG_Cooked);
#else
		false;
#endif
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE 

#if WITH_EDITOR
void UAnimSequenceBase::OnEndLoadPackage(const FEndLoadPackageContext& Context)
{
	if (!GetPackage()->GetHasBeenEndLoaded())
	{
		return;
	}

	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);

	OnAnimModelLoaded();
}

void UAnimSequenceBase::OnAnimModelLoaded()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_OnAnimModelLoaded);
	if(USkeleton* MySkeleton = GetSkeleton())
	{		
		if (IsDataModelValid())
		{
			const TArray<FFloatCurve>& FloatCurves = DataModelInterface->GetFloatCurves();
			for (int32 Index = 0; Index < FloatCurves.Num(); ++Index)
			{
				const FFloatCurve& Curve = FloatCurves[Index];
				ensureMsgf(Curve.GetName() != NAME_None, TEXT("[AnimSequencer %s] has invalid curve name."), *GetFullName());
			}
		}
	}
}
#endif // WITH_EDITOR
