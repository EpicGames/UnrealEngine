// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSequenceHelpers.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimData/IAnimationDataController.h"


#include "Animation/AnimSequence.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimComposite.h"
#include "Animation/MirrorDataTable.h"
#include "BonePose.h"

#include "AnimationRuntime.h"
#include "AnimEncoding.h"
#include "AnimationUtils.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Misc/MessageDialog.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/AnimCurveUtils.h"

#define LOCTEXT_NAMESPACE "AnimSequenceHelpers"

namespace UE
{

namespace Anim
{

#if WITH_EDITOR
void BuildPoseFromModel(const IAnimationDataModel* Model, FAnimationPoseData& OutPoseData, const double Time, const EAnimInterpolationType& InterpolationType, const FName& RetargetSource, const TArray<FTransform>& RetargetTransforms)
{
	FCompactPose& OutPose = OutPoseData.GetPose();
	check(Model);

	OutPose.ResetToRefPose();

	const UE::Anim::DataModel::FEvaluationContext EvaluationContext(Time, Model->GetFrameRate(), RetargetSource, RetargetTransforms, InterpolationType);
	Model->Evaluate(OutPoseData, EvaluationContext);
}

void EvaluateFloatCurvesFromModel(const IAnimationDataModel* Model, FBlendedCurve& OutCurves, double Time)
{
	check(Model);

	// Evaluate into a temporary curve, then filter by enabled curves
	const TArray<FFloatCurve>& ModelCurves = Model->GetFloatCurves();
	const int32 NumCurves = ModelCurves.Num();

	auto GetNameFromIndex = [&ModelCurves](int32 InCurveIndex)
	{
		return ModelCurves[InCurveIndex].GetName();
	};

	auto GetValueFromIndex = [&ModelCurves, Time](int32 InCurveIndex)
	{
		return ModelCurves[InCurveIndex].Evaluate(Time);
	};

	UE::Anim::FCurveUtils::BuildUnsorted(OutCurves, NumCurves, GetNameFromIndex, GetValueFromIndex, OutCurves.GetFilter());
}

void EvaluateTransformCurvesFromModel(const IAnimationDataModel* Model, TMap<FName, FTransform>& OutCurves, double Time, float BlendWeight)
{
	if (Model)
	{
		for (const FTransformCurve& Curve : Model->GetTransformCurves())
		{
			// if disabled, do not handle
			if (Curve.GetCurveTypeFlag(AACF_Disabled))
			{
				continue;
			}

			// Add or retrieve curve
			// note we're not checking Curve.GetCurveTypeFlags() yet
			FTransform& Value = OutCurves.FindOrAdd(Curve.GetName());
			Value = Curve.Evaluate(Time, BlendWeight);
		}
	}
}

void CopyCurveDataToModel(const FRawCurveTracks& CurveData, const USkeleton* Skeleton, IAnimationDataController& Controller)
{
	// Populate float curve data
	for (const FFloatCurve& FloatCurve : CurveData.FloatCurves)
	{
		const FAnimationCurveIdentifier CurveId = UAnimationCurveIdentifierExtensions::FindCurveIdentifier(Skeleton, FloatCurve.GetName(), ERawCurveTrackTypes::RCT_Float);
		if (CurveId.IsValid())
		{
			Controller.AddCurve(CurveId, FloatCurve.GetCurveTypeFlags());
			FCurveAttributes Attributes;
			Attributes.SetPreExtrapolation(FloatCurve.FloatCurve.PreInfinityExtrap);
			Attributes.SetPostExtrapolation(FloatCurve.FloatCurve.PostInfinityExtrap);					
			Controller.SetCurveColor(CurveId, FloatCurve.GetColor());
			Controller.SetCurveAttributes(CurveId, Attributes);
			Controller.SetCurveColor(CurveId, FloatCurve.GetColor());
			Controller.SetCurveKeys(CurveId, FloatCurve.FloatCurve.GetConstRefOfKeys());
		}
	}

	// Populate transform curve data
	for (const FTransformCurve& TransformCurve : CurveData.TransformCurves)
	{
		const FAnimationCurveIdentifier CurveId = UAnimationCurveIdentifierExtensions::FindCurveIdentifier(Skeleton, TransformCurve.GetName(), ERawCurveTrackTypes::RCT_Transform);
		if (CurveId.IsValid())
		{
			Controller.AddCurve(CurveId, TransformCurve.GetCurveTypeFlags());

			// Set each individual channel rich curve keys, to account for any custom tangents etc.
			for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
			{
				const ETransformCurveChannel Channel = (ETransformCurveChannel)SubCurveIndex;
				const FVectorCurve* VectorCurve = TransformCurve.GetVectorCurveByIndex(SubCurveIndex);
				for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
				{
					const EVectorCurveChannel Axis = (EVectorCurveChannel)ChannelIndex;
					FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
					UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
					Controller.SetCurveKeys(TargetCurveIdentifier, VectorCurve->FloatCurves[ChannelIndex].GetConstRefOfKeys());
				}
			}
		}
	}
}
#endif // WITH_EDITOR
 
 
 void ExtractBoneTransform(const FRawAnimSequenceTrack& RawTrack, FTransform& OutTransform, int32 KeyIndex)
{
	// Bail out (with rather wacky data) if data is empty for some reason.
	if (RawTrack.PosKeys.Num() == 0 || RawTrack.RotKeys.Num() == 0)
	{
		UE_LOGF(LogAnimation, Log, "GetBoneTransform : No anim data in track!");
		OutTransform.SetIdentity();
		return;
	}

	const int32 PosKeyIndex = FMath::Min(KeyIndex, RawTrack.PosKeys.Num() - 1);
	const int32 RotKeyIndex = FMath::Min(KeyIndex, RawTrack.RotKeys.Num() - 1);
	static const FVector DefaultScale3D = FVector(1.f);

	OutTransform.SetTranslation(FVector(RawTrack.PosKeys[PosKeyIndex]));
	OutTransform.SetRotation(FQuat(RawTrack.RotKeys[RotKeyIndex]));
	if (RawTrack.ScaleKeys.Num() > 0)
	{
		const int32 ScaleKeyIndex = FMath::Min(KeyIndex, RawTrack.ScaleKeys.Num() - 1);
		OutTransform.SetScale3D(FVector(RawTrack.ScaleKeys[ScaleKeyIndex]));
	}
	else
	{
		OutTransform.SetScale3D(DefaultScale3D);
	}	
}

FTransform MirrorTransform(const FTransform& Transform, const UMirrorDataTable& MirrorDataTable)
{
	FVector T = Transform.GetTranslation();
	T = FAnimationRuntime::MirrorVector(T, MirrorDataTable.MirrorAxis);

	FQuat Q = Transform.GetRotation();
	Q = FAnimationRuntime::MirrorQuat(Q, MirrorDataTable.MirrorAxis);

	const FVector S = Transform.GetScale3D();
	return FTransform(Q, T, S);
}

FTransform ExtractRootMotionFromAnimationAsset(const UAnimationAsset* Animation, const UMirrorDataTable* MirrorDataTable, float StartTime, float DeltaTime, bool bAllowLooping)
{
	if (!Animation)
	{
		return FTransform::Identity;
	}

	// Note: Adapted from 'UAnimSequence::ExtractRootMotion' - since anim sequence doesn't know about MirrorDataTable
	FRootMotionMovementParams RootMotionParams;

	if (DeltaTime != 0.f)
	{
		bool const bPlayingBackwards = (DeltaTime < 0.f);

		float PreviousPosition = StartTime;
		float CurrentPosition = StartTime;
		float DesiredDeltaMove = DeltaTime;

		do
		{
			// Disable looping here. Advance to desired position, or beginning / end of animation 
			const ETypeAdvanceAnim AdvanceType = FAnimationRuntime::AdvanceTime(false, DesiredDeltaMove, CurrentPosition, Animation->GetPlayLength());

			// Verify position assumptions
			ensureMsgf(bPlayingBackwards ? (CurrentPosition <= PreviousPosition) : (CurrentPosition >= PreviousPosition), TEXT("in Animation %s(Skeleton %s) : bPlayingBackwards(%d), PreviousPosition(%0.2f), Current Position(%0.2f)"),
				*Animation->GetName(), *GetNameSafe(Animation->GetSkeleton()), bPlayingBackwards, PreviousPosition, CurrentPosition);
			
			RootMotionParams.Accumulate(UE::Anim::ExtractRootMotionFromAnimationAsset(Animation, MirrorDataTable, PreviousPosition, CurrentPosition));

			// If we've hit the end of the animation, and we're allowed to loop, keep going.
			if ((AdvanceType == ETAA_Finished) && bAllowLooping)
			{
				const double ActualDeltaMove = (CurrentPosition - PreviousPosition);
				DesiredDeltaMove -= ActualDeltaMove;

				PreviousPosition = bPlayingBackwards ? Animation->GetPlayLength() : 0.f;
				CurrentPosition = PreviousPosition;
			}
			else
			{
				break;
			}
		} while (true);
	}

	return RootMotionParams.GetRootMotionTransform();
}

FTransform ExtractRootMotionFromAnimationAsset(const UAnimationAsset* Animation, const UMirrorDataTable* MirrorDataTable, float StartPosition, float EndPosition)
{
	FTransform Result = FTransform::Identity;
	
	if (const UAnimMontage* Montage = Cast<UAnimMontage>(Animation))
	{
		Result = Montage->ExtractRootMotionFromTrackRange(StartPosition, EndPosition, FAnimExtractContext());
	}
	else if (const UAnimComposite* AnimComposite = Cast<UAnimComposite>(Animation))
	{
		FRootMotionMovementParams RootMotion;
		AnimComposite->ExtractRootMotionFromTrack(AnimComposite->AnimationTrack, StartPosition, EndPosition, FAnimExtractContext(), RootMotion);
		Result = RootMotion.GetRootMotionTransform();
	}
	else if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Animation))
	{
		Result = AnimSequence->ExtractRootMotionFromRange(StartPosition, EndPosition, FAnimExtractContext());
	}

	if (MirrorDataTable)
	{
		Result =  MirrorTransform(Result, *MirrorDataTable);
	}

	return Result;
}

FTransform ExtractRootTransformFromAnimationAsset(const UAnimationAsset* Animation, float Time)
{
	const FVector DefaultScale(1.f);
	bool bUseNormalizedRootMotionScale = false;

	FTransform RootTransformRefPose = FTransform::Identity;
	if (Animation)
	{
		if (const USkeleton* MySkeleton = Animation->GetSkeleton())
		{
			const FReferenceSkeleton& RefSkeleton = MySkeleton->GetReferenceSkeleton();
			if (RefSkeleton.GetNum() > 0)
			{
				RootTransformRefPose = RefSkeleton.GetRefBonePose()[0];
			}
		}
	}

	FTransform Result = FTransform::Identity;
	if (const UAnimMontage* AnimMontage = Cast<UAnimMontage>(Animation))
	{
		if (const FAnimSegment* Segment = AnimMontage->SlotAnimTracks[0].AnimTrack.GetSegmentAtTime(Time))
		{
			if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Segment->GetAnimReference()))
			{
				const float AnimSequenceTime = Segment->ConvertTrackPosToAnimPos(Time);
				Result = AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(static_cast<double>(AnimSequenceTime)), nullptr);
				bUseNormalizedRootMotionScale = AnimSequence->bUseNormalizedRootMotionScale;
			}	
		}
	}
	else if (const UAnimComposite* AnimComposite = Cast<UAnimComposite>(Animation))
	{
		Result = AnimComposite->ExtractRootTrackTransform(FAnimExtractContext(static_cast<double>(Time)), nullptr);
	}
	else if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Animation))
	{
		Result = AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(static_cast<double>(Time)), nullptr);
		bUseNormalizedRootMotionScale = AnimSequence->bUseNormalizedRootMotionScale;
	}

	// Use old calculation if needed.
	if (bUseNormalizedRootMotionScale)
	{
		//Clear scale as it will muck up GetRelativeTransform
		Result.SetScale3D(FVector(1.f));
	}
	else if (Animation)
	{
		if (Animation->IsValidAdditive())
		{
			Result.SetScale3D(Result.GetScale3D() + DefaultScale);
		}
	}

	// Transform to Component Space
	const FTransform RootToComponent = RootTransformRefPose.Inverse();
	Result = RootToComponent * Result;

	return Result;
}

Retargeting::FRetargetingScope::FRetargetingScope(const USkeleton* InSourceSkeleton, FCompactPose& ToRetargetPose, const DataModel::FEvaluationContext& InEvaluationContext)
	: SourceSkeleton(InSourceSkeleton),
	RetargetPose(ToRetargetPose),
	EvaluationContext(InEvaluationContext),
	RetargetTracking(FBuildRawPoseScratchArea::Get().RetargetTracking),
	bShouldRetarget((!ToRetargetPose.GetBoneContainer().GetDisableRetargeting() && EvaluationContext.RetargetTransforms.Num()) || (SourceSkeleton != ToRetargetPose.GetBoneContainer().GetSkeletonAsset()))
{
	check(SourceSkeleton);
	RetargetTracking.Reset();
}

void Retargeting::FRetargetingScope::AddTrackedBone(FCompactPoseBoneIndex CompactBoneIndex, int32 SkeletonBoneIndex) const
{
	if (bShouldRetarget)
	{
		RetargetTracking.Add({CompactBoneIndex, SkeletonBoneIndex});
	}
}

Retargeting::FRetargetingScope::~FRetargetingScope()
{
	if (bShouldRetarget)
	{
		const FBoneContainer& RequiredBones = RetargetPose.GetBoneContainer();
		for (const FRetargetTracking& RT : RetargetTracking)
		{
			FAnimationRuntime::RetargetBoneTransform(SourceSkeleton, EvaluationContext.RetargetSource, EvaluationContext.RetargetTransforms, RetargetPose[RT.PoseBoneIndex], RT.SkeletonBoneIndex, RT.PoseBoneIndex, RequiredBones, false);
		}
	}
}

#if WITH_EDITOR
bool CopyNotifies(const UAnimSequenceBase* SourceAnimSeq, UAnimSequenceBase* DestAnimSeq, bool bShowDialogs, bool bDeleteNotifies)
{
	// Abort if source == destination.
	if (SourceAnimSeq == DestAnimSeq)
	{
		return true;
	}

	// If the destination sequence is shorter than the source sequence, we'll be dropping notifies that
	// occur at later times than the dest sequence is long.  Give the user a chance to abort if we
	// find any notifies that won't be copied over.
	if (DestAnimSeq->GetPlayLength() < SourceAnimSeq->GetPlayLength())
	{
		for (int32 NotifyIndex = 0; NotifyIndex < SourceAnimSeq->Notifies.Num(); ++NotifyIndex)
		{
			// If a notify is found which occurs off the end of the destination sequence, prompt the user to continue.
			const FAnimNotifyEvent& SrcNotifyEvent = SourceAnimSeq->Notifies[NotifyIndex];
			if (SrcNotifyEvent.GetTriggerTime() > DestAnimSeq->GetPlayLength())
			{
				UE_LOGF(LogAnimation, Warning, "Animation Notify trigger time %f falls outside of the destination animation sequence its length %f, notify will not be copied.", SrcNotifyEvent.GetTriggerTime(), DestAnimSeq->GetPlayLength());
				
				const bool bProceed = !bShowDialogs || EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, NSLOCTEXT("UnrealEd", "SomeNotifiesWillNotBeCopiedQ", "Some notifies will not be copied because the destination sequence is not long enough.  Proceed?"));
				if (!bProceed)
				{
					return false;
				}
				else
				{
					break;
				}
			}
		}
	}

	// If the destination sequence contains any notifies, ask the user if they'd like
	// to delete the existing notifies before copying over from the source sequence.
	if (DestAnimSeq->Notifies.Num() > 0)
	{
		const bool bDeleteExistingNotifies = bDeleteNotifies || (bShowDialogs && EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(
			NSLOCTEXT("UnrealEd", "DestSeqAlreadyContainsNotifiesMergeQ", "The destination sequence already contains {0} notifies.  Delete these before copying?"), FText::AsNumber(DestAnimSeq->Notifies.Num()))));

		if (bDeleteExistingNotifies)
		{
			DestAnimSeq->Notifies.Empty();
			DestAnimSeq->MarkPackageDirty();
			DestAnimSeq->RefreshCacheData();
		}
	}

	// Do the copy.
	int32 NumNotifiesThatWereNotCopied = 0;

	for (int32 NotifyIndex = 0; NotifyIndex < SourceAnimSeq->Notifies.Num(); ++NotifyIndex)
	{
		const FAnimNotifyEvent& SrcNotifyEvent = SourceAnimSeq->Notifies[NotifyIndex];

		// Skip notifies which occur at times later than the destination sequence is long.
		if (SrcNotifyEvent.GetTriggerTime() > DestAnimSeq->GetPlayLength())
		{
			++NumNotifiesThatWereNotCopied;
			continue;
		}

		// Copy notify tracks from src to dest if they are missing
		if (SrcNotifyEvent.TrackIndex >= DestAnimSeq->AnimNotifyTracks.Num())
		{
			for (int32 TrackIndex = DestAnimSeq->AnimNotifyTracks.Num(); TrackIndex <= SrcNotifyEvent.TrackIndex; ++TrackIndex)
			{
				DestAnimSeq->AnimNotifyTracks.Add(FAnimNotifyTrack(SourceAnimSeq->AnimNotifyTracks[TrackIndex].TrackName, SourceAnimSeq->AnimNotifyTracks[TrackIndex].TrackColor));
			}
		}

		// Track the location of the new notify.
		int32 NewNotifyIndex = DestAnimSeq->Notifies.AddDefaulted();
		FAnimNotifyEvent& NotifyEvent = DestAnimSeq->Notifies[NewNotifyIndex];

		// Copy properties of the NotifyEvent
		NotifyEvent.TrackIndex = SrcNotifyEvent.TrackIndex;
		NotifyEvent.NotifyName = SrcNotifyEvent.NotifyName;
		NotifyEvent.Duration = SrcNotifyEvent.Duration;

		// Copy the notify itself, and point the new one at it.
		if (SrcNotifyEvent.Notify)
		{
			DestAnimSeq->Notifies[NewNotifyIndex].Notify = static_cast<UAnimNotify*>(StaticDuplicateObject(SrcNotifyEvent.Notify, DestAnimSeq, NAME_None, RF_AllFlags, nullptr, EDuplicateMode::Normal, ~EInternalObjectFlags::RootSet));
		}
		else
		{
			DestAnimSeq->Notifies[NewNotifyIndex].Notify = nullptr;
		}

		if (SrcNotifyEvent.NotifyStateClass)
		{
			DestAnimSeq->Notifies[NewNotifyIndex].NotifyStateClass = static_cast<UAnimNotifyState*>(StaticDuplicateObject(SrcNotifyEvent.NotifyStateClass, DestAnimSeq, NAME_None, RF_AllFlags, nullptr, EDuplicateMode::Normal, ~EInternalObjectFlags::RootSet));
		}
		else
		{
			DestAnimSeq->Notifies[NewNotifyIndex].NotifyStateClass = nullptr;
		}

		// Copy notify timing
		NotifyEvent.Link(DestAnimSeq, SrcNotifyEvent.GetTriggerTime());
		NotifyEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(DestAnimSeq->CalculateOffsetForNotify(NotifyEvent.GetTriggerTime()));

		// Make sure editor knows we've changed something.
		DestAnimSeq->MarkPackageDirty();
		DestAnimSeq->RefreshCacheData();
	}

	// Inform the user if some notifies weren't copied.
	if (bShowDialogs && NumNotifiesThatWereNotCopied > 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(
			NSLOCTEXT("UnrealEd", "SomeNotifiesWereNotCopiedF", "Because the destination sequence was shorter, {0} notifies were not copied."), FText::AsNumber(NumNotifiesThatWereNotCopied)));
	}

	return true;
}

#endif // WITH_EDITOR

bool Compression::CompressAnimationDataTracks(const USkeleton* Skeleton, const TArray<FTrackToSkeletonMap>& TrackToSkeleton, TArray<FRawAnimSequenceTrack>& RawAnimationData, int32 NumberOfKeys, FName ErrorName, float MaxPosDiff, float MaxAngleDiff, float MaxScaleDiff)
{
	bool bRemovedKeys = false;

#if WITH_EDITORONLY_DATA
	if (ensureMsgf(RawAnimationData.Num() > 0, TEXT("%s is trying to compress while raw animation is missing"), * ErrorName.ToString()))
	{
		// This removes trivial keys, and this has to happen before the removing tracks
		for (int32 TrackIndex = 0; TrackIndex < RawAnimationData.Num(); TrackIndex++)
		{
			bRemovedKeys |= CompressRawAnimSequenceTrack(RawAnimationData[TrackIndex], NumberOfKeys, ErrorName, MaxPosDiff, MaxAngleDiff, MaxScaleDiff);
		}
	}
#endif
	return bRemovedKeys;
}

bool Compression::CompressRawAnimSequenceTrack(FRawAnimSequenceTrack& RawTrack, int32 NumberOfKeys, FName ErrorName, float MaxPosDiff, float MaxAngleDiff, float MaxScaleDiff)
{
	bool bRemovedKeys = false;

	// First part is to make sure we have valid input
	bool const bPosTrackIsValid = (RawTrack.PosKeys.Num() == 1 || RawTrack.PosKeys.Num() == NumberOfKeys);
	if (!bPosTrackIsValid)
	{
		UE_LOGF(LogAnimation, Warning, "Found non valid position track for %ls, %d frames, instead of %d. Chopping!", *ErrorName.ToString(), RawTrack.PosKeys.Num(), NumberOfKeys);
		bRemovedKeys = true;
		RawTrack.PosKeys.RemoveAt(1, RawTrack.PosKeys.Num() - 1);
		RawTrack.PosKeys.Shrink();
		check(RawTrack.PosKeys.Num() == 1);
	}

	bool const bRotTrackIsValid = (RawTrack.RotKeys.Num() == 1 || RawTrack.RotKeys.Num() == NumberOfKeys);
	if (!bRotTrackIsValid)
	{
		UE_LOGF(LogAnimation, Warning, "Found non valid rotation track for %ls, %d frames, instead of %d. Chopping!", *ErrorName.ToString(), RawTrack.RotKeys.Num(), NumberOfKeys);
		bRemovedKeys = true;
		RawTrack.RotKeys.RemoveAt(1, RawTrack.RotKeys.Num() - 1);
		RawTrack.RotKeys.Shrink();
		check(RawTrack.RotKeys.Num() == 1);
	}

	// scale keys can be empty, and that is valid 
	bool const bScaleTrackIsValid = (RawTrack.ScaleKeys.Num() == 0 || RawTrack.ScaleKeys.Num() == 1 || RawTrack.ScaleKeys.Num() == NumberOfKeys);
	if (!bScaleTrackIsValid)
	{
		UE_LOGF(LogAnimation, Warning, "Found non valid scaling track for %ls, %d frames, instead of %d. Chopping!", *ErrorName.ToString(), RawTrack.ScaleKeys.Num(), NumberOfKeys);
		bRemovedKeys = true;
		RawTrack.ScaleKeys.RemoveAt(1, RawTrack.ScaleKeys.Num() - 1);
		RawTrack.ScaleKeys.Shrink();
		check(RawTrack.ScaleKeys.Num() == 1);
	}

	// Second part is actual compression.

	// Check variation of position keys
	if ((RawTrack.PosKeys.Num() > 1) && (MaxPosDiff >= 0.0f))
	{
		FVector3f FirstPos = RawTrack.PosKeys[0];
		bool bFramesIdentical = true;
		for (int32 j = 1; j < RawTrack.PosKeys.Num() && bFramesIdentical; j++)
		{
			if ((FirstPos - RawTrack.PosKeys[j]).SizeSquared() > FMath::Square(MaxPosDiff))
			{
				bFramesIdentical = false;
			}
		}

		// If all keys are the same, remove all but first frame
		if (bFramesIdentical)
		{
			bRemovedKeys = true;
			RawTrack.PosKeys.RemoveAt(1, RawTrack.PosKeys.Num() - 1);
			RawTrack.PosKeys.Shrink();
			check(RawTrack.PosKeys.Num() == 1);
		}
	}

	// Check variation of rotational keys
	if ((RawTrack.RotKeys.Num() > 1) && (MaxAngleDiff >= 0.0f))
	{
		FQuat4f FirstRot = RawTrack.RotKeys[0];
		bool bFramesIdentical = true;
		for (int32 j = 1; j < RawTrack.RotKeys.Num() && bFramesIdentical; j++)
		{
			if (FQuat4f::Error(FirstRot, RawTrack.RotKeys[j]) > MaxAngleDiff)
			{
				bFramesIdentical = false;
			}
		}

		// If all keys are the same, remove all but first frame
		if (bFramesIdentical)
		{
			bRemovedKeys = true;
			RawTrack.RotKeys.RemoveAt(1, RawTrack.RotKeys.Num() - 1);
			RawTrack.RotKeys.Shrink();
			check(RawTrack.RotKeys.Num() == 1);
		}
	}

	// Check variation of Scaleition keys
	if ((RawTrack.ScaleKeys.Num() > 1) && (MaxScaleDiff >= 0.0f))
	{
		FVector3f FirstScale = RawTrack.ScaleKeys[0];
		bool bFramesIdentical = true;
		for (int32 j = 1; j < RawTrack.ScaleKeys.Num() && bFramesIdentical; j++)
		{
			if ((FirstScale - RawTrack.ScaleKeys[j]).SizeSquared() > FMath::Square(MaxScaleDiff))
			{
				bFramesIdentical = false;
			}
		}

		// If all keys are the same, remove all but first frame
		if (bFramesIdentical)
		{
			bRemovedKeys = true;
			RawTrack.ScaleKeys.RemoveAt(1, RawTrack.ScaleKeys.Num() - 1);
			RawTrack.ScaleKeys.Shrink();
			check(RawTrack.ScaleKeys.Num() == 1);
		}
	}

	return bRemovedKeys;
}

void Compression::SanitizeRawAnimSequenceTrack(FRawAnimSequenceTrack& RawTrack)
{
	// if scale is too small, zero it out. Cause it hard to retarget when compress
	// inverse scale is applied to translation, and causing translation to be huge to retarget, but
	// compression can't handle that much precision. 
	for (auto& Scale3D : RawTrack.ScaleKeys)
	{
		if (FMath::IsNearlyZero(Scale3D.X))
		{
			Scale3D.X = 0.f;
		}
		if (FMath::IsNearlyZero(Scale3D.Y))
		{
			Scale3D.Y = 0.f;
		}
		if (FMath::IsNearlyZero(Scale3D.Z))
		{
			Scale3D.Z = 0.f;
		}
	}
	
	for (auto& Position : RawTrack.PosKeys)
	{		
		if (FMath::IsNearlyZero(Position.X))
		{
			Position.X = 0.f;
		}
		if (FMath::IsNearlyZero(Position.Y))
		{
			Position.Y = 0.f;
		}
		if (FMath::IsNearlyZero(Position.Z))
		{
			Position.Z = 0.f;
		}
	}

	// make sure Rotation part is normalized before compress
	for (auto& Rotation : RawTrack.RotKeys)
	{
		if (!Rotation.IsNormalized())
		{
			Rotation.Normalize();
		}

		if (FMath::IsNearlyZero(Rotation.X))
		{
			Rotation.X = 0.f;
		}
		if (FMath::IsNearlyZero(Rotation.Y))
		{
			Rotation.Y = 0.f;
		}
		if (FMath::IsNearlyZero(Rotation.Z))
		{
			Rotation.Z = 0.f;
		}
	}
}

#if WITH_EDITOR
Compression::FScopedCompressionGuard::FScopedCompressionGuard(UAnimSequence* InAnimSequence) : AnimSequence(InAnimSequence)
{
	if(AnimSequence)
	{
		AnimSequence->bBlockCompressionRequests = true;		
	}
}

Compression::FScopedCompressionGuard::~FScopedCompressionGuard()
{
	if(AnimSequence)
	{
		AnimSequence->bBlockCompressionRequests = false;		
	}
}

bool AnimationData::AddLoopingInterpolation(UAnimSequence* InSequence)
{
	const IAnimationDataModel* DataModel = InSequence->GetDataModel();
	IAnimationDataController& Controller = InSequence->GetController();

	const int32 NumTracks = DataModel->GetNumBoneTracks();
	const int32 NumKeys = DataModel->GetNumberOfKeys();

	if (NumTracks > 0 && NumKeys > 0)
	{
		// now I need to calculate back to new animation data
		auto LoopKeyData = [&](auto& KeyData)
		{
			// Need at least a single 
			if (KeyData.Num() > 1)
			{
				auto FirstKey = KeyData[0];
				KeyData.Add(FirstKey);
			}
		};
		
		IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("AddLoopingInterpolation_Bracket", "Adding looping interpolation"));

		TArray<FTransform> BoneTransforms;
		TArray<FName> TrackNames;
		DataModel->GetBoneTrackNames(TrackNames);
 
		TArray<FVector3f> PositionalKeys, ScaleKeys;
		TArray<FQuat4f> RotationalKeys;
		for (const FName& TrackName : TrackNames)
		{
			BoneTransforms.Reset();
			PositionalKeys.Reset();
			RotationalKeys.Reset();
			ScaleKeys.Reset();
			DataModel->GetBoneTrackTransforms(TrackName, BoneTransforms);

			for (const FTransform& Transform : BoneTransforms)
			{
				PositionalKeys.Add(FVector3f(Transform.GetLocation()));
				RotationalKeys.Add(FQuat4f(Transform.GetRotation()));
				ScaleKeys.Add(FVector3f(Transform.GetScale3D()));
			}
		
			LoopKeyData(PositionalKeys);
			LoopKeyData(RotationalKeys);
			LoopKeyData(ScaleKeys);

			Controller.SetBoneTrackKeys(TrackName, PositionalKeys, RotationalKeys, ScaleKeys);
		}

		// New number of frames is equal to current number of keys, as we'll be adding one frame (Number of Frames + 1 == Number of Keys)
		Controller.SetNumberOfFrames(NumKeys);

		return true;
	}

	return false;
}

bool AnimationData::Trim(UAnimSequence* InSequence, TRange<FFrameNumber> TrimRange)
{
	const IAnimationDataModel* DataModel = InSequence->GetDataModel();
	IAnimationDataController& Controller = InSequence->GetController();

	const int32 NumTracks = DataModel->GetNumBoneTracks();
	const int32 NumKeys = DataModel->GetNumberOfKeys();

	if (NumTracks > 0 && NumKeys > 0)
	{
		const FFrameRate& FrameRate = DataModel->GetFrameRate();
		// if there is only one key, there is nothing to trim away
		if (NumKeys <= 1)
		{
			return false;
		}

		// Convert frame range to key indices
		const int32 StartTrimFrameIndex = TrimRange.GetLowerBound().IsInclusive() ? TrimRange.GetLowerBound().GetValue().Value : TrimRange.GetLowerBound().GetValue().Value + 1;
		const int32 EndTrimFrameIndex = TrimRange.GetUpperBound().IsInclusive() ? TrimRange.GetUpperBound().GetValue().Value : TrimRange.GetUpperBound().GetValue().Value - 1;
		const int32 NumTrimmedFrames = (EndTrimFrameIndex - StartTrimFrameIndex) + 1;

		const int32 StartTrimKeyIndex = StartTrimFrameIndex;
		const int32 NumTrimmedKeys = NumTrimmedFrames;

		if (StartTrimKeyIndex == 0 && NumTrimmedFrames == NumKeys)
		{
			return false;
		}

		if (NumTrimmedFrames == 0)
		{
			return false;
		}
		
		IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("TrimRawAnimation_Bracket", "Trimming Animation Track Data"));
		RemoveKeys(InSequence, StartTrimKeyIndex, NumTrimmedFrames);
		
		return true;
	}

	return false;
}


void AnimationData::DuplicateKeys(UAnimSequence* InSequence, int32 StartKeyIndex, int32 NumDuplicates, int32 SourceKeyIndex /*= INDEX_NONE */)
{
	const IAnimationDataModel* Model = InSequence->GetDataModel();
	IAnimationDataController& Controller = InSequence->GetController();

	const int32 NumberOfKeys = Model->GetNumberOfKeys();
	const FFrameRate& FrameRate = Model->GetFrameRate();

	// Ensure that the index at which keys will be inserted, and the source key index for the duplicates is valid as well
	if (StartKeyIndex >= 0 && StartKeyIndex <= NumberOfKeys && NumDuplicates >= 1)
	{
		const int32 CopyKeyIndex = SourceKeyIndex == INDEX_NONE ? StartKeyIndex : SourceKeyIndex;
		if (CopyKeyIndex >= 0 && CopyKeyIndex < NumberOfKeys)
		{
			IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("DuplicateKeys_Bracket", "Inserting Duplicate Animation Track Keys"));

			const int32 NumFramesToInsert = NumDuplicates;
			const int32 EndFrameIndex = StartKeyIndex + NumDuplicates;

			auto InsertFrames = [&](auto& KeyData)
			{
				if (KeyData.Num() >= 1 && KeyData.IsValidIndex(CopyKeyIndex))
				{
					auto SourceKeyData = KeyData[CopyKeyIndex];
					KeyData.InsertZeroed(StartKeyIndex, NumFramesToInsert);

					for (int32 FrameIndex = StartKeyIndex; FrameIndex < EndFrameIndex; ++FrameIndex)
					{
						KeyData[FrameIndex] = SourceKeyData;
					}
				}
			};

			TArray<FName> TrackNames;
			Model->GetBoneTrackNames(TrackNames);
			
			TArray<FTransform> BoneTransforms;
			BoneTransforms.Reserve(Model->GetNumberOfKeys());
			
			for (const FName& TrackName : TrackNames)
			{
				BoneTransforms.Reset();
				Model->GetBoneTrackTransforms(TrackName, BoneTransforms);

				TArray<FVector3f> PosKeys;
				PosKeys.SetNum(BoneTransforms.Num() + NumDuplicates);
				TArray<FQuat4f> RotKeys;
				RotKeys.SetNum(BoneTransforms.Num() + NumDuplicates);
				TArray<FVector3f> ScaleKeys;
				ScaleKeys.SetNum(BoneTransforms.Num() + NumDuplicates);

				ensure(BoneTransforms.IsValidIndex(SourceKeyIndex));
				for (int32 TransformIndex = 0; TransformIndex < BoneTransforms.Num() + NumFramesToInsert; ++TransformIndex)
				{
					if (TransformIndex < StartKeyIndex)
					{
						PosKeys[TransformIndex] = FVector3f(BoneTransforms[TransformIndex].GetLocation());
						RotKeys[TransformIndex] = FQuat4f(BoneTransforms[TransformIndex].GetRotation());
						ScaleKeys[TransformIndex] = FVector3f(BoneTransforms[TransformIndex].GetScale3D());
					}
					if (TransformIndex >= StartKeyIndex && TransformIndex < StartKeyIndex + NumDuplicates)
					{
						PosKeys[TransformIndex] = FVector3f(BoneTransforms[SourceKeyIndex].GetLocation());
						RotKeys[TransformIndex] = FQuat4f(BoneTransforms[SourceKeyIndex].GetRotation());
						ScaleKeys[TransformIndex] = FVector3f(BoneTransforms[SourceKeyIndex].GetScale3D());
					}
					else if (TransformIndex >= StartKeyIndex + NumDuplicates)
					{
						PosKeys[TransformIndex] = FVector3f(BoneTransforms[TransformIndex - NumDuplicates].GetLocation());
						RotKeys[TransformIndex] = FQuat4f(BoneTransforms[TransformIndex - NumDuplicates].GetRotation());
						ScaleKeys[TransformIndex] = FVector3f(BoneTransforms[TransformIndex - NumDuplicates].GetScale3D());
					}
				}

				Controller.SetBoneTrackKeys(TrackName, PosKeys, RotKeys, ScaleKeys);
				
				BoneTransforms.Reset();
			}

			// The number of keys has changed, which means that the sequence length and number of frames should be updated as well
			const int32 NewNumKeys = NumberOfKeys + NumDuplicates;
			const int32 NewNumFrames = NewNumKeys - 1;
			const float NewSequenceLength = FrameRate.AsSeconds(NewNumFrames);

			const float StartTime = FrameRate.AsSeconds(StartKeyIndex);
			const float InsertedTime = FrameRate.AsInterval() * NumDuplicates;

			// Notify will happen with time slice that was inserted
			Controller.ResizeInFrames(NewNumFrames, StartKeyIndex, StartKeyIndex + NumDuplicates);
		}
	}
}

void AnimationData::RemoveKeys(UAnimSequence* InSequence, int32 StartKeyIndex, int32 NumKeysToRemove)
{
	const IAnimationDataModel* Model = InSequence->GetDataModel();
	IAnimationDataController& Controller = InSequence->GetController();

	const int32 NumberOfKeys = Model->GetNumberOfKeys();
	const FFrameRate& FrameRate = Model->GetFrameRate();

	const int32 EndKeyIndex = StartKeyIndex + NumKeysToRemove;
	if (StartKeyIndex >= 0 && StartKeyIndex < NumberOfKeys && NumKeysToRemove > 0 && EndKeyIndex <= NumberOfKeys)
	{
		IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("RemoveKeys_Bracket", "Removing Animation Track Keys"));

		auto ShrinkKeys = [&](auto& KeyData)
		{
			// Dont allow us to trim below 2 keys (1 frame)
			int32 NumKeyDataToRemove = NumKeysToRemove;
			if((KeyData.Num() - NumKeysToRemove) < 2)
			{
				NumKeyDataToRemove = KeyData.Num() - 2;
			}

			if (KeyData.Num() >= (StartKeyIndex + NumKeyDataToRemove))
			{
				KeyData.RemoveAt(StartKeyIndex, NumKeyDataToRemove);
				check(KeyData.Num() > 0);
				KeyData.Shrink();
			}
		};

		const int32 NewNumberOfKeys = NumberOfKeys - NumKeysToRemove;

		TArray<FName> TrackNames;
		Model->GetBoneTrackNames(TrackNames);
			
		TArray<FTransform> BoneTransforms;
		BoneTransforms.Reserve(Model->GetNumberOfKeys());
			
		for (const FName& TrackName : TrackNames)
		{
			BoneTransforms.Reset();

			Model->GetBoneTrackTransforms(TrackName, BoneTransforms);

			TArray<FVector3f> PosKeys;
			PosKeys.SetNum(BoneTransforms.Num() - NumKeysToRemove);
			TArray<FQuat4f> RotKeys;
			RotKeys.SetNum(BoneTransforms.Num() - NumKeysToRemove);
			TArray<FVector3f> ScaleKeys;
			ScaleKeys.SetNum(BoneTransforms.Num() - NumKeysToRemove);

			for (int32 TransformIndex = 0; TransformIndex < BoneTransforms.Num() - NumKeysToRemove; ++TransformIndex)
			{
				if (TransformIndex < StartKeyIndex)
				{
					PosKeys[TransformIndex] = FVector3f(BoneTransforms[TransformIndex].GetLocation());
					RotKeys[TransformIndex] = FQuat4f(BoneTransforms[TransformIndex].GetRotation());
					ScaleKeys[TransformIndex] = FVector3f(BoneTransforms[TransformIndex].GetScale3D());
				}
				if (TransformIndex >= StartKeyIndex)
				{
					PosKeys[TransformIndex] = FVector3f(BoneTransforms[TransformIndex + NumKeysToRemove].GetLocation());
					RotKeys[TransformIndex] = FQuat4f(BoneTransforms[TransformIndex + NumKeysToRemove].GetRotation());
					ScaleKeys[TransformIndex] = FVector3f(BoneTransforms[TransformIndex + NumKeysToRemove].GetScale3D());		
				}
			}
			
			Controller.SetBoneTrackKeys(TrackName, PosKeys, RotKeys, ScaleKeys);
		}

		const int32 NewNumberOfFrames = FMath::Max(NewNumberOfKeys - 1, 1);

		const int32 StartFrame = FMath::Max(StartKeyIndex - 1, 0);
		// Notify will happen with time slice that was removed
		Controller.ResizeInFrames(NewNumberOfFrames, StartFrame, StartFrame + NumKeysToRemove);
	}
}

FName AnimationData::FindFirstChildTrackName(const UAnimSequence* InSequence, const USkeleton* Skeleton, const FName& BoneName)
{
	const IAnimationDataModel* DataModel = InSequence->GetDataModel();
	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

	const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		// get out, nothing to do
		return NAME_None;
	}

	// find children
	TArray<int32> ChildBoneIndices;
	if (Skeleton->GetChildBones(BoneIndex, ChildBoneIndices) > 0)
	{
		// first look for direct children
		for (const int32 ChildBoneIndex : ChildBoneIndices)
		{
			const FName ChildBoneName = RefSkeleton.GetBoneName(ChildBoneIndex);
			if (DataModel->IsValidBoneTrackName(ChildBoneName))
			{
				// found the new track
				return ChildBoneName;
			}
		}

		int32 BestGrandChildIndex = INDEX_NONE;
		FName BestGrandChildName = NAME_None;
		// if you didn't find yet, now you have to go through all children
		for (const int32 ChildBoneIndex : ChildBoneIndices)
		{
			const FName ChildBoneName = RefSkeleton.GetBoneName(ChildBoneIndex);
			// now I have to go through all childrewn and find who is earliest since I don't know which one might be the closest one
			const FName GrandChildName = FindFirstChildTrackName(InSequence, Skeleton, ChildBoneName);
			if (GrandChildName != NAME_None)
			{
				const int32 GrandChildIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(GrandChildName);
				if (BestGrandChildIndex == INDEX_NONE)
				{
					BestGrandChildIndex = GrandChildIndex;
					BestGrandChildName = GrandChildName;
				}
				else if (BestGrandChildIndex > GrandChildIndex)
				{
					// best should be earlier track index
					BestGrandChildIndex = GrandChildIndex;
					BestGrandChildName = GrandChildName;
				}
			}
		}

		return BestGrandChildName;
	}

	return NAME_None;
}

#endif // WITH_EDITOR


} // Namespace Anim

} // Namespace UE

#undef LOCTEXT_NAMESPACE 
