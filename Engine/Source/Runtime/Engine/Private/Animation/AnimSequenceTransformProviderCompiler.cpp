// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceTransformProviderCompiler.h"

#if WITH_EDITOR

#include "Animation/AnimSequenceTransformProviderData.h"
#include "Animation/AnimationSequenceCompiler.h"
#include "AnimationUtils.h"
#include "AsyncCompilationHelpers.h"
#include "Async/AsyncWork.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "AssetCompilingManager.h"
#include "SkinnedAssetCompiler.h"
#include "AnimationRuntime.h"
#include "BoneContainer.h"
#include "ReferenceSkeleton.h"
#include "Algo/NoneOf.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Misc/QueuedThreadPool.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "ObjectCacheContext.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Serialization/MemoryHasher.h"

#define LOCTEXT_NAMESPACE "AnimSequenceTransformProviderCompiler"

static TAutoConsoleVariable<int32> CVarMemoryForAnimSequenceTransformProviderCompile(
	TEXT("Memory.MemoryForAnimSequenceTransformProviderCompile"),
	256,
	TEXT("Memory in MiB set aside for an AnimSequenceTransformProvider asset compile job\n"),
	ECVF_Default);

static AsyncCompilationHelpers::FAsyncCompilationStandardCVars CVarAsyncAnimSequenceTransformProviderStandard(
	TEXT("AnimSequenceTransformProvider"),
	TEXT("animsequence transform provider"),
	FConsoleCommandDelegate::CreateLambda(
	[]()
	{
		FAnimSequenceTransformProviderCompilingManager::Get().FinishAllCompilation();
	}
));

namespace AnimSequenceTransformProviderCompilingManagerImpl
{
	static void EnsureInitializedCVars()
	{
		static bool bIsInitialized = false;

		if (!bIsInitialized)
		{
			bIsInitialized = true;

			AsyncCompilationHelpers::EnsureInitializedCVars(
				TEXT("AnimSequenceTransformProvider"),
				CVarAsyncAnimSequenceTransformProviderStandard.AsyncCompilation,
				CVarAsyncAnimSequenceTransformProviderStandard.AsyncCompilationMaxConcurrency);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FAnimSequenceTransformProviderCompilingManager::FAnimSequenceTransformProviderCompilingManager()
	: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	AnimSequenceTransformProviderCompilingManagerImpl::EnsureInitializedCVars();
	PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddRaw(
		this, &FAnimSequenceTransformProviderCompilingManager::OnPostReachabilityAnalysis);
}

void FAnimSequenceTransformProviderCompilingManager::OnPostReachabilityAnalysis()
{
	if (GetNumRemainingAssets())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProviderCompilingManager::CancelUnreachableProviders);

		TArray<UAnimSequenceTransformProviderData*> PendingProviders;
		PendingProviders.Reserve(GetNumRemainingAssets());

		for (auto Iterator = RegisteredProviders.CreateIterator(); Iterator; ++Iterator)
		{
			UAnimSequenceTransformProviderData* Provider = Iterator->GetEvenIfUnreachable();
			if (Provider && Provider->IsUnreachable())
			{
				UE_LOGF(LogAnimation, Verbose, "Cancelling transform provider %ls compilation because it's being garbage collected", *Provider->GetName());

				if (Provider->TryCancelAsyncTasks())
				{
					Iterator.RemoveCurrent();
				}
				else
				{
					PendingProviders.Add(Provider);
				}
			}
		}

		FinishCompilation(PendingProviders);
	}
}

FName FAnimSequenceTransformProviderCompilingManager::GetAssetTypeName() const
{
	return TEXT("UE-AnimSequenceTransformProvider");
}

FTextFormat FAnimSequenceTransformProviderCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("AssetNameFormat", "{0}|plural(one=Anim Sequence Transform Provider,other=Anim Sequence Transform Providers)");
}

TArrayView<FName> FAnimSequenceTransformProviderCompilingManager::GetDependentTypeNames() const
{
	static FName DependentTypeNames[] =
	{
		// AnimSequenceTransformProvider can wait on AnimSequence to finish their own compilation before compiling itself
		// so they need to be processed before us. This is especially important when FinishAllCompilation is issued
		// so that we know once we're called that all anim sequences have finished compiling.
		UE::Anim::FAnimSequenceCompilingManager::GetStaticAssetTypeName()
	};
	return TArrayView<FName>(DependentTypeNames);
}

EQueuedWorkPriority FAnimSequenceTransformProviderCompilingManager::GetBasePriority(UAnimSequenceTransformProviderData* InProvider) const
{
	return EQueuedWorkPriority::Low;
}

FQueuedThreadPool* FAnimSequenceTransformProviderCompilingManager::GetThreadPool() const
{
	static FQueuedThreadPoolDynamicWrapper* GTransformProviderThreadPool = nullptr;
	if (GTransformProviderThreadPool == nullptr && FAssetCompilingManager::Get().GetThreadPool() != nullptr)
	{
		// Transform providers will be scheduled on the asset thread pool, where concurrency limits might by dynamically adjusted depending on memory constraints.
		GTransformProviderThreadPool = new FQueuedThreadPoolDynamicWrapper(FAssetCompilingManager::Get().GetThreadPool(), -1, [](EQueuedWorkPriority) { return EQueuedWorkPriority::Low; });

		AsyncCompilationHelpers::BindThreadPoolToCVar(
			GTransformProviderThreadPool,
			CVarAsyncAnimSequenceTransformProviderStandard.AsyncCompilation,
			CVarAsyncAnimSequenceTransformProviderStandard.AsyncCompilationResume,
			CVarAsyncAnimSequenceTransformProviderStandard.AsyncCompilationMaxConcurrency
		);
	}

	return GTransformProviderThreadPool;
}

void FAnimSequenceTransformProviderCompilingManager::Shutdown()
{
	bHasShutdown = true;
	if (GetNumRemainingAssets())
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProviderCompilingManager::Shutdown);

		TArray<UAnimSequenceTransformProviderData*> PendingProviders;
		PendingProviders.Reserve(GetNumRemainingAssets());

		for (TWeakObjectPtr<UAnimSequenceTransformProviderData>& WeakProvider : RegisteredProviders)
		{
			if (WeakProvider.IsValid())
			{
				UAnimSequenceTransformProviderData* Provider = WeakProvider.Get();
				if (!Provider->TryCancelAsyncTasks())
				{
					PendingProviders.Add(Provider);
				}
			}
		}

		FinishCompilation(PendingProviders);
	}

	FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostReachabilityAnalysisHandle);
}

FAnimSequenceTransformProviderCompilingManager& FAnimSequenceTransformProviderCompilingManager::Get()
{
	static FAnimSequenceTransformProviderCompilingManager Singleton;
	return Singleton;
}

int32 FAnimSequenceTransformProviderCompilingManager::GetNumRemainingAssets() const
{
	return RegisteredProviders.Num();
}

TRACE_DECLARE_INT_COUNTER(QueuedAnimSequenceTransformProviderCompilation, TEXT("AsyncCompilation/QueuedAnimSequenceTransformProvider"));
void FAnimSequenceTransformProviderCompilingManager::UpdateCompilationNotification()
{
	TRACE_COUNTER_SET(QueuedAnimSequenceTransformProviderCompilation, GetNumRemainingAssets());
	Notification->Update(GetNumRemainingAssets());
}

void FAnimSequenceTransformProviderCompilingManager::AddProvider(UAnimSequenceTransformProviderData* Provider)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProviderCompilingManager::AddProvider);
	check(IsInGameThread());

	RegisteredProviders.Emplace(Provider);

	FObjectCacheContextScope ObjectCacheScope;
	for (UInstancedSkinnedMeshComponent* Component : ObjectCacheScope.GetContext().GetInstancedSkinnedMeshComponents(Provider->GetRootTransformProvider()))
	{
		Component->PreAssetCompilation();
	}

	TRACE_COUNTER_SET(QueuedAnimSequenceTransformProviderCompilation, GetNumRemainingAssets());
}

void FAnimSequenceTransformProviderCompilingManager::FinishCompilation(TArrayView<UAnimSequenceTransformProviderData* const> Providers)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProviderCompilingManager::FinishCompilation);

	// Allow calls from any thread if the providers are already finished compiling.
	if (Algo::NoneOf(Providers, &UAnimSequenceTransformProviderData::IsCompiling))
	{
		return;
	}

	check(IsInGameThread());

	TArray<UAnimSequenceTransformProviderData*> PendingProviders;
	PendingProviders.Reserve(Providers.Num());

	for (UAnimSequenceTransformProviderData* Provider : Providers)
	{
		if (RegisteredProviders.Contains(Provider))
		{
			PendingProviders.Emplace(Provider);
		}
	}

	if (PendingProviders.Num())
	{
		class FCompilableProvider : public AsyncCompilationHelpers::ICompilable
		{
		public:
			FCompilableProvider(UAnimSequenceTransformProviderData* InProvider)
				: Provider(InProvider)
			{
			}

			void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority)
			{
				// No-op: rescheduling not implemented for this asset type
			}

			bool WaitCompletionWithTimeout(float TimeLimitSeconds) override
			{
				return Provider->WaitForAsyncTasks(TimeLimitSeconds);
			}

			UAnimSequenceTransformProviderData* Provider;
			FName GetName() override { return Provider->GetOutermost()->GetFName(); }
		};

		TArray<FCompilableProvider> CompilableProviders(PendingProviders);
		FObjectCacheContextScope ObjectCacheScope;
		AsyncCompilationHelpers::FinishCompilation(
			[&CompilableProviders](int32 Index) -> AsyncCompilationHelpers::ICompilable& { return CompilableProviders[Index]; },
			CompilableProviders.Num(),
			LOCTEXT("AnimSequenceTransformProviders", "Anim Sequence Transform Providers"),
			LogAnimation,
			[this](AsyncCompilationHelpers::ICompilable* Object)
			{
				UAnimSequenceTransformProviderData* Provider = static_cast<FCompilableProvider*>(Object)->Provider;
				PostCompilation(Provider);
				RegisteredProviders.Remove(Provider);
			}
		);

		PostCompilation(PendingProviders);
	}
}

void FAnimSequenceTransformProviderCompilingManager::FinishAllCompilation()
{
	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProviderCompilingManager::FinishAllCompilation);

	if (GetNumRemainingAssets())
	{
		TArray<UAnimSequenceTransformProviderData*> PendingProviders;
		PendingProviders.Reserve(GetNumRemainingAssets());

		for (TWeakObjectPtr<UAnimSequenceTransformProviderData>& WeakProvider : RegisteredProviders)
		{
			if (WeakProvider.IsValid())
			{
				PendingProviders.Add(WeakProvider.Get());
			}
		}

		FinishCompilation(PendingProviders);
	}
}

void FAnimSequenceTransformProviderCompilingManager::PostCompilation(TArrayView<UAnimSequenceTransformProviderData* const> Providers)
{
	if (Providers.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OnAssetPostCompileEvent);

		TArray<FAssetCompileData> AssetsData;
		AssetsData.Reserve(Providers.Num());

		for (UAnimSequenceTransformProviderData* Provider : Providers)
		{
			// Do not broadcast an event for unreachable objects
			if (!Provider->IsUnreachable())
			{
				AssetsData.Emplace(Provider);
			}
		}

		if (AssetsData.Num())
		{
			FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast(AssetsData);
		}
	}
}

void FAnimSequenceTransformProviderCompilingManager::PostCompilation(UAnimSequenceTransformProviderData* Provider)
{
	using namespace AnimSequenceTransformProviderCompilingManagerImpl;

	// If AsyncTask is null here, the task got canceled so we don't need to do anything
	if (!IsEngineExitRequested())
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(PostCompilation);

		Provider->FinishCacheDerivedData();

		// Do not do anything else if the Provider is being garbage collected
		if (Provider->IsUnreachable())
		{
			return;
		}

		FObjectCacheContextScope ObjectCacheScope;
		for (UInstancedSkinnedMeshComponent* Component : ObjectCacheScope.GetContext().GetInstancedSkinnedMeshComponents(Provider->GetRootTransformProvider()))
		{
			Component->PostAssetCompilation();
		}

		// Calling this delegate during app exit might be quite dangerous and lead to crash
		// if the content browser wants to refresh a thumbnail it might try to load a package
		// which will then fail due to various reasons related to the editor shutting down.
		// Triggering this callback while garbage collecting can also result in listeners trying to look up objects
		if (!GExitPurge && !IsGarbageCollecting())
		{
			// Generate an empty property changed event, to force the asset registry tag
			// to be refreshed now that cached data is available.
			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Provider, EmptyPropertyChangedEvent);
		}
	}
}

void FAnimSequenceTransformProviderCompilingManager::ProcessProviders(bool bLimitExecutionTime, int32 MinBatchSize)
{
	using namespace AnimSequenceTransformProviderCompilingManagerImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProviderCompilingManager::ProcessProviders);

	const int32 NumRemainingProviders = GetNumRemainingAssets();

	// Spread out the load over multiple frames but if too many providers, convergence is more important than frame time
	const int32 MaxProviderUpdatesPerFrame = bLimitExecutionTime ? FMath::Max(64, NumRemainingProviders / 10) : INT32_MAX;

	FObjectCacheContextScope ObjectCacheScope;
	if (NumRemainingProviders && NumRemainingProviders >= MinBatchSize)
	{
		TSet<UAnimSequenceTransformProviderData*> ProvidersToProcess;
		for (TWeakObjectPtr<UAnimSequenceTransformProviderData>& Provider : RegisteredProviders)
		{
			if (Provider.IsValid())
			{
				ProvidersToProcess.Add(Provider.Get());
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessFinishedProviders);

			TSet<TWeakObjectPtr<UAnimSequenceTransformProviderData>> ProvidersToPostpone;
			TArray<UAnimSequenceTransformProviderData*> ProcessedProviders;

			if (ProvidersToProcess.Num())
			{
				for (UAnimSequenceTransformProviderData* Provider : ProvidersToProcess)
				{
					const bool bHasProviderUpdateLeft = ProcessedProviders.Num() <= MaxProviderUpdatesPerFrame;
					if (bHasProviderUpdateLeft && Provider->IsAsyncTaskComplete())
					{
						PostCompilation(Provider);
						ProcessedProviders.Add(Provider);
					}
					else
					{
						ProvidersToPostpone.Emplace(Provider);
					}
				}
			}

			RegisteredProviders = MoveTemp(ProvidersToPostpone);
			PostCompilation(ProcessedProviders);
		}
	}
}

void FAnimSequenceTransformProviderCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	FObjectCacheContextScope ObjectCacheScope;
	ProcessProviders(bLimitExecutionTime);
	UpdateCompilationNotification();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FAnimSequenceTransformProviderAsyncBuildWorker : public FNonAbandonableTask
{
	FAnimSequenceTransformProviderBuildAsyncCacheTask* Owner;
	FIoHash IoHash;

public:
	FAnimSequenceTransformProviderAsyncBuildWorker(FAnimSequenceTransformProviderBuildAsyncCacheTask* InOwner, const FIoHash& InIoHash)
		: Owner(InOwner)
		, IoHash(InIoHash)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAnimSequenceTransformProviderAsyncBuildWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	void DoWork();
};

struct FAnimSequenceTransformProviderAsyncBuildTask : public FAsyncTask<FAnimSequenceTransformProviderAsyncBuildWorker>
{
	FAnimSequenceTransformProviderAsyncBuildTask(FAnimSequenceTransformProviderBuildAsyncCacheTask* InOwner, const FIoHash& InIoHash)
		: FAsyncTask<FAnimSequenceTransformProviderAsyncBuildWorker>(InOwner, InIoHash)
	{
	}
};

class FAnimSequenceTransformProviderBuildAsyncCacheTask
{
public:
	FAnimSequenceTransformProviderBuildAsyncCacheTask(
		const FIoHash& InKeyHash,
		FAnimSequenceTransformProviderCachedData* InData,
		UAnimSequenceTransformProviderData& InProvider,
		const ITargetPlatform* InTargetPlatform)
		: Data(InData)
		, WeakProvider(&InProvider)
		, Sequences(InProvider.GetSequences())
		, SkinnedAsset(InProvider.GetSkinnedAsset())
		, TargetPlatform(InTargetPlatform)
		// Once we pass the BeginCache throttling gate, we want to finish as fast as possible
		// to avoid holding on to memory for a long time. We use the high priority since it will go fast,
		// but also it will avoid starving the critical threads in the subsequent task.
		, Owner(UE::DerivedData::EPriority::High)
		, bIsWaitingOnCompilation(ShouldWaitForCompilation())
		, KeyHash(InKeyHash)
	{
		/**
		 * Unfortunately our async builds are not made to handle the assets that use data from other assets
		 * This will delay the start of the actual cache until the build of the sequences is done
		 * This will fix a race condition with the sequence build without blocking the game thread by default.
		 * Note: This is not a perfect solution since it also delays the DDC data pull.
		 */
		if (!bIsWaitingOnCompilation)
		{
			BeginCache(InKeyHash);
		}
	}

	inline void Wait()
	{
		if (bIsWaitingOnCompilation)
		{
			WaitForDependenciesAndBeginCache();
		}

		if (BuildTask != nullptr)
		{
			BuildTask->EnsureCompletion();
		}

		Owner.Wait();
	}

	inline bool WaitWithTimeout(float TimeLimitSeconds)
	{
		if (bIsWaitingOnCompilation)
		{
			if (!WaitForDependenciesAndBeginCacheWithTimeout(TimeLimitSeconds))
			{
				return false;
			}
		}

		if (BuildTask != nullptr && !BuildTask->WaitCompletionWithTimeout(TimeLimitSeconds))
		{
			return false;
		}

		return Owner.Poll();
	}

	inline bool Poll()
	{
		if (bIsWaitingOnCompilation)
		{
			BeginCacheIfDependenciesAreFree();
		}

		if (BuildTask && !BuildTask->IsDone())
		{
			return false;
		}

		return Owner.Poll();
	}

	inline void Cancel()
	{
		// Cancel the waiting on the build
		bIsWaitingOnCompilation = false;

		if (BuildTask)
		{
			BuildTask->Cancel();
		}

		Owner.Cancel();
	}

private:
	bool ShouldWaitForCompilation() const;

	void BeginCacheIfDependenciesAreFree();
	void WaitForDependenciesAndBeginCache();
	bool WaitForDependenciesAndBeginCacheWithTimeout(float TimeLimitSeconds);

	void BeginCache(const FIoHash& InKeyHash);
	void EndCache(UE::DerivedData::FCacheGetValueResponse&& Response);
	bool BuildData(const UE::FSharedString& Name, const UE::DerivedData::FCacheKey& Key);

private:
	friend class FAnimSequenceTransformProviderAsyncBuildWorker;
	TUniquePtr<FAnimSequenceTransformProviderAsyncBuildTask> BuildTask;
	FAnimSequenceTransformProviderCachedData* Data;
	TWeakObjectPtr<UAnimSequenceTransformProviderData> WeakProvider;
	TArray<FAnimSequenceTransformProviderSequence> Sequences;
	TObjectPtr<USkinnedAsset> SkinnedAsset;
	const ITargetPlatform* TargetPlatform = nullptr;
	UE::DerivedData::FRequestOwner Owner;
	TRefCountPtr<IExecutionResource> ExecutionResource;
	bool bIsWaitingOnCompilation;
	FIoHash KeyHash;
};

void FAnimSequenceTransformProviderAsyncBuildWorker::DoWork()
{
	using namespace UE::DerivedData;
	if (UAnimSequenceTransformProviderData* Provider = Owner->WeakProvider.Get())
	{
		// Grab any execution resources currently assigned to this worker so that we maintain
		// concurrency limit and memory pressure until the whole multi-step task is done.
		Owner->ExecutionResource = FExecutionResourceContext::Get();

		static const FCacheBucket Bucket("AnimSequenceTransformProvider");
		GetCache().GetValue({ {{Provider->GetPathName()}, {Bucket, IoHash}} }, Owner->Owner,
			[Task = Owner](FCacheGetValueResponse&& Response)
			{
				Task->EndCache(MoveTemp(Response));
			}
		);
	}
}

bool FAnimSequenceTransformProviderBuildAsyncCacheTask::ShouldWaitForCompilation() const
{
	if (IsValid(SkinnedAsset))
	{
		if (SkinnedAsset->HasAnyFlags(RF_NeedPostLoad) || SkinnedAsset->IsCompiling())
		{
			return true;
		}
	}

	for (const FAnimSequenceTransformProviderSequence& SequenceData : Sequences)
	{
		if (!IsValid(SequenceData.Sequence))
		{
			continue;
		}

		// If the sequence is still waiting for a post load call, let it build its stuff first to avoid blocking the Game Thread
		if (SequenceData.Sequence->HasAnyFlags(RF_NeedPostLoad) || SequenceData.Sequence->IsCompiling() || !SequenceData.Sequence->CanBeCompressed())
		{
			return true;
		}
	}
	return false;
}

void FAnimSequenceTransformProviderBuildAsyncCacheTask::BeginCacheIfDependenciesAreFree()
{
	if (UAnimSequenceTransformProviderData* Provider = WeakProvider.Get())
	{
		if (!ShouldWaitForCompilation())
		{
			bIsWaitingOnCompilation = false;
			BeginCache(KeyHash);
		}
	}
	else
	{
		bIsWaitingOnCompilation = false;
	}
}

void FAnimSequenceTransformProviderBuildAsyncCacheTask::WaitForDependenciesAndBeginCache()
{
	if (IsValid(SkinnedAsset))
	{
		if (SkinnedAsset->HasAnyFlags(RF_NeedPostLoad))
		{
			SkinnedAsset->ConditionalPostLoad();
		}

		FSkinnedAssetCompilingManager::Get().FinishCompilation({ SkinnedAsset });
	}

	for (const FAnimSequenceTransformProviderSequence& SequenceData : Sequences)
	{
		if (!SequenceData.Sequence)
		{
			continue;
		}
		if (SequenceData.Sequence->HasAnyFlags(RF_NeedPostLoad))
		{
			SequenceData.Sequence->ConditionalPostLoad();
		}

		UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation({ SequenceData.Sequence.Get() });
	}

	bIsWaitingOnCompilation = false;
	BeginCache(KeyHash);
}

bool FAnimSequenceTransformProviderBuildAsyncCacheTask::WaitForDependenciesAndBeginCacheWithTimeout(float TimeLimitSeconds)
{
	for (const FAnimSequenceTransformProviderSequence& SequenceData : Sequences)
	{
		if (!SequenceData.Sequence || !SequenceData.Sequence->IsCompiling())
		{
			continue;
		}
		if (!SequenceData.Sequence->WaitForAsyncTasks(TimeLimitSeconds))
		{
			return false;
		}
	}

	// Performs any necessary cleanup now that the async task (if any) is complete
	WaitForDependenciesAndBeginCache();

	return true;
}

void FAnimSequenceTransformProviderBuildAsyncCacheTask::BeginCache(const FIoHash& InKeyHash)
{
	using namespace UE::DerivedData;

	if (UAnimSequenceTransformProviderData* Provider = WeakProvider.Get())
	{
		for (const FAnimSequenceTransformProviderSequence& SequenceData : Sequences)
		{
			UAnimSequence* Sequence = SequenceData.Sequence;
			if (!IsValid(Sequence))
			{
				continue;
			}
			Sequence->BeginCacheDerivedData(TargetPlatform);
		}

		// Queue this launch through the thread pool so that we benefit from fair scheduling and memory throttling
		FQueuedThreadPool* ThreadPool = FAnimSequenceTransformProviderCompilingManager::Get().GetThreadPool();
		EQueuedWorkPriority BasePriority = FAnimSequenceTransformProviderCompilingManager::Get().GetBasePriority(Provider);

		int64 RequiredMemory = 1024 * 1024 * CVarMemoryForAnimSequenceTransformProviderCompile.GetValueOnAnyThread();

		check(BuildTask == nullptr);
		BuildTask = MakeUnique<FAnimSequenceTransformProviderAsyncBuildTask>(this, InKeyHash);
		BuildTask->StartBackgroundTask(ThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, RequiredMemory, TEXT("AnimSequenceTransformProvider"));
	}
}

void FAnimSequenceTransformProviderBuildAsyncCacheTask::EndCache(UE::DerivedData::FCacheGetValueResponse&& Response)
{
	using namespace UE::DerivedData;

	if (Response.Status == EStatus::Ok)
	{
		Owner.LaunchTask(TEXT("AnimSequenceTransformProviderSerialize"), [this, Value = MoveTemp(Response.Value)]
		{
			// Release execution resource as soon as the task is done
			ON_SCOPE_EXIT { ExecutionResource = nullptr; };

			if (WeakProvider.Get())
			{
				FSharedBuffer RecordData = Value.GetData().Decompress();
				FMemoryReaderView Ar(RecordData, /*bIsPersistent*/ true);
				Ar << *Data;
			}
		});
	}
	else if (Response.Status == EStatus::Error)
	{
		Owner.LaunchTask(TEXT("AnimSequenceTransformProviderBuild"), [this, Name = Response.Name, Key = Response.Key]
		{
			// Release execution resource as soon as the task is done
			ON_SCOPE_EXIT { ExecutionResource = nullptr; };

			if (!BuildData(Name, Key))
			{
				return;
			}

			if (WeakProvider.Get())
			{
				TArray64<uint8> RecordData;
				FMemoryWriter64 Ar(RecordData, /*bIsPersistent*/ true);
				Ar << *Data;

				GetCache().PutValue({ {Name, Key, FValue::Compress(MakeSharedBufferFromArray(MoveTemp(RecordData)))} }, Owner);
			}
		});
	}
	else
	{
		// Release execution resource as soon as the task is done
		ExecutionResource = nullptr;
	}
}

bool FAnimSequenceTransformProviderBuildAsyncCacheTask::BuildData(const UE::FSharedString& Name, const UE::DerivedData::FCacheKey& Key)
{
	*Data = {};

	if (!IsValid(SkinnedAsset))
	{
		return false;
	}

	const FBoxSphereBounds AssetBounds = SkinnedAsset->GetBounds();
	Data->SequenceBounds.SetNum(Sequences.Num());

	for (int32 SequenceIndex = 0; SequenceIndex < Sequences.Num(); ++SequenceIndex)
	{
		if (Owner.IsCanceled())
		{
			return false;
		}

		const FAnimSequenceTransformProviderSequence& SequenceData = Sequences[SequenceIndex];
		UAnimSequence* Sequence = SequenceData.Sequence;

		if (!IsValid(Sequence))
		{
			Data->SequenceBounds[SequenceIndex] = AssetBounds;
			continue;
		}

		Sequence->FinishAsyncTasks();

		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProviderBuildAsyncCacheTask::BuildSequence);
		const FReferenceSkeleton& MeshRefSkeleton = SkinnedAsset->GetRefSkeleton();
		const int32 NumMeshBones = MeshRefSkeleton.GetRawBoneNum();

		FVector3f AnimatedBoundsMin(AssetBounds.Origin - AssetBounds.BoxExtent);
		FVector3f AnimatedBoundsMax(AssetBounds.Origin + AssetBounds.BoxExtent);

		const int32 NumFrames = FMath::Max(1, Sequence->GetSamplingFrameRate().AsFrameTime(Sequence->GetPlayLength()).RoundToFrame().Value);

		TArray<FBoneIndexType> BoneIndices;
		BoneIndices.SetNumUninitialized(NumMeshBones);
		for (int32 Index = 0; Index < NumMeshBones; ++Index)
		{
			BoneIndices[Index] = (FBoneIndexType)Index;
		}

		UPhysicsAsset* PhysicsAsset = SkinnedAsset->GetPhysicsAsset();

		FBoneContainer BoneContainer(BoneIndices, UE::Anim::FCurveFilterSettings(), *SkinnedAsset);
		const TArray<FTransform>& MeshLocalRefPose = MeshRefSkeleton.GetRawRefBonePose();
		TArray<FTransform> SampledGlobalMeshPose;
		SampledGlobalMeshPose.SetNumUninitialized(NumMeshBones);

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			// Some paths in the decompression code use mem stack, so make sure we put a mark here.
			FMemMark Mark(FMemStack::Get());

			const double SeekTime = Sequence->GetSamplingFrameRate().AsSeconds(FFrameTime(FrameIndex));

			FCompactPose AnimPose;
			AnimPose.SetBoneContainer(&BoneContainer);
			FBlendedCurve AnimCurve;
			AnimCurve.InitFrom(BoneContainer);
			UE::Anim::FStackAttributeContainer AnimAttributes;

			FAnimExtractContext ExtractionContext(SeekTime);
			FAnimationPoseData AnimationPoseData(AnimPose, AnimCurve, AnimAttributes);
			Sequence->GetAnimationPose(AnimationPoseData, ExtractionContext);

			for (int32 MeshBoneIndex = 0; MeshBoneIndex < NumMeshBones; ++MeshBoneIndex)
			{
				const FCompactPoseBoneIndex CompactIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
				const int32 ParentIndex = MeshRefSkeleton.GetParentIndex(MeshBoneIndex);
				if (SampledGlobalMeshPose.IsValidIndex(ParentIndex) && ParentIndex < MeshBoneIndex)
				{
					SampledGlobalMeshPose[MeshBoneIndex] = AnimationPoseData.GetPose()[CompactIndex] * SampledGlobalMeshPose[ParentIndex];
				}
				else
				{
					SampledGlobalMeshPose[MeshBoneIndex] = MeshLocalRefPose[MeshBoneIndex];
				}
			}

			if (PhysicsAsset)
			{
				for (int32 BodyIdx : PhysicsAsset->BoundsBodies)
				{
					USkeletalBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIdx];
					int32 BoneIndex = MeshRefSkeleton.FindBoneIndex(BodySetup->BoneName);

					if (BoneIndex != INDEX_NONE && BoneIndex < SampledGlobalMeshPose.Num())
					{
						FTransform BoneTransform = SampledGlobalMeshPose[BoneIndex];
						FBox BodyBounds = BodySetup->AggGeom.CalcAABB(BoneTransform);

						AnimatedBoundsMin = FVector3f::Min(AnimatedBoundsMin, FVector3f(BodyBounds.Min));
						AnimatedBoundsMax = FVector3f::Max(AnimatedBoundsMax, FVector3f(BodyBounds.Max));
					}
				}
			}
			else
			{
				for (int32 MeshBoneIndex = 0; MeshBoneIndex < NumMeshBones; ++MeshBoneIndex)
				{
					const FVector3f BonePosition = FVector3f(SampledGlobalMeshPose[MeshBoneIndex].GetTranslation());
					AnimatedBoundsMin = FVector3f::Min(AnimatedBoundsMin, BonePosition);
					AnimatedBoundsMax = FVector3f::Max(AnimatedBoundsMax, BonePosition);
				}
			}
		}

		Data->SequenceBounds[SequenceIndex] = FBoxSphereBounds(FBox(FVector(AnimatedBoundsMin), FVector(AnimatedBoundsMax)));
	}

	if (Owner.IsCanceled())
	{
		return false;
	}

	// Compute conservative bounds as the union of all sequence bounds.
	Data->ConservativeBounds = AssetBounds;
	for (const FBoxSphereBounds& Bounds : Data->SequenceBounds)
	{
		Data->ConservativeBounds = Data->ConservativeBounds + Bounds;
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FIoHash UAnimSequenceTransformProviderData::CreateDerivedDataKeyHash(const ITargetPlatform* TargetPlatform)
{
	FMemoryHasherBlake3 Writer;
	FGuid VersionGuid(0xA3F7B2E1, 0x4D8C6F9A, 0xB84E5562, 0xC9D1E3A9);
	Writer << VersionGuid;

	if (IsValid(SkinnedAsset))
	{
		FString AssetHash = SkinnedAsset->BuildDerivedDataKey(TargetPlatform);
		Writer << AssetHash;
	}

	for (int32 SequenceIndex = 0; SequenceIndex < Sequences.Num(); ++SequenceIndex)
	{
		Writer << SequenceIndex;
		const FAnimSequenceTransformProviderSequence& SequenceData = Sequences[SequenceIndex];
		if (IsValid(SequenceData.Sequence))
		{
			FAnimationUtils::EnsureAnimSequenceLoaded(*SequenceData.Sequence);
			FIoHash SequenceHash = SequenceData.Sequence->GetDerivedDataKeyHash(TargetPlatform);
			Writer << SequenceHash;
		}
	}

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	FString ArmSuffix("_arm64");
	Writer << ArmSuffix;
#endif

	return Writer.Finalize();
}

FIoHash UAnimSequenceTransformProviderData::BeginCacheDerivedData(const ITargetPlatform* TargetPlatform)
{
	// Skip bounds cooking if no skinned asset. Can't compute bounds without reference skeleton.
	if (!IsValid(SkinnedAsset))
	{
		return FIoHash();
	}

	// Instances use parent's cached data. No need to compute their own.
	if (IsA<UAnimSequenceTransformProviderDataInstance>())
	{
		return FIoHash();
	}

	const FIoHash KeyHash = CreateDerivedDataKeyHash(TargetPlatform);

	if (KeyHash.IsZero() || DataKeyHash == KeyHash || DataByPlatformKeyHash.Contains(KeyHash))
	{
		return KeyHash;
	}

	if (CacheTasksByKeyHash.Contains(KeyHash))
	{
		return KeyHash;
	}

	CachedData = {};

	FAnimSequenceTransformProviderCachedData* TargetData = nullptr;
	if (TargetPlatform->IsRunningPlatform())
	{
		DataKeyHash = KeyHash;
		TargetData = &CachedData;
	}
	else
	{
		TargetData = DataByPlatformKeyHash.Emplace(KeyHash, MakeUnique<FAnimSequenceTransformProviderCachedData>()).Get();
	}

	CacheTasksByKeyHash.Emplace(KeyHash, MakePimpl<FAnimSequenceTransformProviderBuildAsyncCacheTask>(KeyHash, TargetData, *this, TargetPlatform));

	FAnimSequenceTransformProviderCompilingManager::Get().AddProvider(this);

	return KeyHash;
}

bool UAnimSequenceTransformProviderData::PollCacheDerivedData(const FIoHash& KeyHash) const
{
	if (KeyHash.IsZero())
	{
		return true;
	}

	if (const TPimplPtr<FAnimSequenceTransformProviderBuildAsyncCacheTask>* Task = CacheTasksByKeyHash.Find(KeyHash))
	{
		return (*Task)->Poll();
	}

	return true;
}

bool UAnimSequenceTransformProviderData::PollCacheDerivedData() const
{
	for (const auto& Pair : CacheTasksByKeyHash)
	{
		if (!Pair.Value->Poll())
		{
			return false;
		}
	}
	return true;
}

void UAnimSequenceTransformProviderData::EndCacheDerivedData(const FIoHash& KeyHash)
{
	if (KeyHash.IsZero())
	{
		return;
	}

	TPimplPtr<FAnimSequenceTransformProviderBuildAsyncCacheTask> Task;
	if (CacheTasksByKeyHash.RemoveAndCopyValue(KeyHash, Task))
	{
		Task->Wait();
	}
}

FAnimSequenceTransformProviderCachedData* UAnimSequenceTransformProviderData::CacheDerivedData(const ITargetPlatform* TargetPlatform)
{
	if (IsCompiling())
	{
		FAnimSequenceTransformProviderCompilingManager::Get().FinishCompilation({ this });
	}

	const FIoHash KeyHash = BeginCacheDerivedData(TargetPlatform);
	EndCacheDerivedData(KeyHash);

	if (DataKeyHash == KeyHash)
	{
		return &CachedData;
	}

	if (TUniquePtr<FAnimSequenceTransformProviderCachedData>* Data = DataByPlatformKeyHash.Find(KeyHash))
	{
		return Data->Get();
	}

	return nullptr;
}

void UAnimSequenceTransformProviderData::FinishCacheDerivedData()
{
	for (auto It = CacheTasksByKeyHash.CreateIterator(); It; ++It)
	{
		It->Value->Wait();
		It.RemoveCurrent();
	}
}

void UAnimSequenceTransformProviderData::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	Super::BeginCacheForCookedPlatformData(TargetPlatform);
	BeginCacheDerivedData(TargetPlatform);
}

bool UAnimSequenceTransformProviderData::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	const FIoHash KeyHash = CreateDerivedDataKeyHash(TargetPlatform);
	if (KeyHash.IsZero())
	{
		return true;
	}

	if (PollCacheDerivedData(KeyHash))
	{
		EndCacheDerivedData(KeyHash);
		return true;
	}

	return false;
}

void UAnimSequenceTransformProviderData::ClearAllCachedCookedPlatformData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimSequenceTransformProviderData::ClearAllCachedCookedPlatformData);

	if (!TryCancelAsyncTasks())
	{
		FinishCacheDerivedData();
	}

	/**
	 * TryCancelAsyncTasks or FinishCacheDerivedData should have been able to clear all tasks. If any tasks remain
	 * then they must still be running, and we would crash when attempting to delete them.
	 */
	check(CacheTasksByKeyHash.IsEmpty());

	DataByPlatformKeyHash.Empty();
	Super::ClearAllCachedCookedPlatformData();
}

bool UAnimSequenceTransformProviderData::TryCancelAsyncTasks()
{
	bool bHadCachedTaskForRunningPlatform = CacheTasksByKeyHash.Contains(DataKeyHash);

	for (auto It = CacheTasksByKeyHash.CreateIterator(); It; ++It)
	{
		if (It->Value->Poll())
		{
			It.RemoveCurrent();
		}
		else
		{
			It->Value->Cancel();

			// Try to see if we can remove the task now that it might have been canceled
			if (It->Value->Poll())
			{
				It.RemoveCurrent();
			}
		}
	}

	if (bHadCachedTaskForRunningPlatform && !CacheTasksByKeyHash.Contains(DataKeyHash))
	{
		// Reset the cached Key for the running platform since we won't have any data
		DataKeyHash = FIoHash();
	}

	return CacheTasksByKeyHash.IsEmpty();
}

bool UAnimSequenceTransformProviderData::IsAsyncTaskComplete() const
{
	for (auto& Pair : CacheTasksByKeyHash)
	{
		if (!Pair.Value->Poll())
		{
			return false;
		}
	}

	return true;
}

bool UAnimSequenceTransformProviderData::WaitForAsyncTasks(float TimeLimitSeconds)
{
	double StartTimeSeconds = FPlatformTime::Seconds();
	for (auto& Pair : CacheTasksByKeyHash)
	{
		// Clamp to 0 as it implies polling
		const float TimeLimit = FMath::Max(0.0f, TimeLimitSeconds - static_cast<float>(FPlatformTime::Seconds() - StartTimeSeconds));
		if (!Pair.Value->WaitWithTimeout(TimeLimit))
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
