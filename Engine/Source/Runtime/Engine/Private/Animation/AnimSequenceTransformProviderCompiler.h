// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "IAssetCompilingManager.h"
#include "UObject/WeakObjectPtr.h"

class FAsyncCompilationNotification;
class UAnimSequenceTransformProviderData;
class FQueuedThreadPool;
enum class EQueuedWorkPriority : uint8;

class FAnimSequenceTransformProviderCompilingManager : public IAssetCompilingManager
{
public:
	static FAnimSequenceTransformProviderCompilingManager& Get();

	/**
	 * Returns the number of outstanding compilations.
	 */
	int32 GetNumRemainingAssets() const override;

	/**
	 * Queue providers to be compiled asynchronously so they are monitored.
	 */
	void AddProvider(UAnimSequenceTransformProviderData* Provider);

	/**
	 * Blocks until completion of the requested providers.
	 */
	void FinishCompilation(TArrayView<UAnimSequenceTransformProviderData* const> Providers);

	/**
	 * Blocks until completion of all async provider compilation.
	 */
	void FinishAllCompilation() override;

	/**
	 * Returns the priority at which the given provider should be scheduled.
	 */
	EQueuedWorkPriority GetBasePriority(UAnimSequenceTransformProviderData* InProvider) const;

	/**
	 * Returns the thread pool where provider compilation should be scheduled.
	 */
	FQueuedThreadPool* GetThreadPool() const;

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	void Shutdown() override;

private:
	friend class FAssetCompilingManager;

	FAnimSequenceTransformProviderCompilingManager();

	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;

	void PostCompilation(UAnimSequenceTransformProviderData* Provider);
	void PostCompilation(TArrayView<UAnimSequenceTransformProviderData* const> Providers);

	void ProcessProviders(bool bLimitExecutionTime, int32 MinBatchSize = 1);
	void UpdateCompilationNotification();

	void OnPostReachabilityAnalysis();
	FDelegateHandle PostReachabilityAnalysisHandle;

	TSet<TWeakObjectPtr<UAnimSequenceTransformProviderData>> RegisteredProviders;
	TUniquePtr<FAsyncCompilationNotification> Notification;
	bool bHasShutdown = false;
};

#endif // WITH_EDITOR
