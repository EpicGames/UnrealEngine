// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSequenceTransformProviderData.h"
#include "SkinningSceneExtensionProxy.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Animation/BlendSpace.h"
#include "SceneInterface.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "ObjectCacheContext.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Misc/DataValidation.h"
#include "Misc/MessageDialog.h"

#if WITH_EDITOR
#include "Animation/AnimBank.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif

#define LOCTEXT_NAMESPACE "AnimSequenceTranformProviderData"

///////////////////////////////////////////////////////////////////////////////////////////////////

static TAutoConsoleVariable<bool> CVarAnimSequenceTransformProviderBlending(
	TEXT("r.AnimSequenceTransformProvider.Blending"),
	true,
	TEXT(""),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

///////////////////////////////////////////////////////////////////////////////////////////////////

void FAnimSequenceTrackPackedData::Serialize(FArchive& Ar)
{
	constexpr auto MakeBitMask = [](uint32 Bits) -> uint32 { return (Bits >= 32) ? 0xffffffffu : ((1u << Bits) - 1u); };

	constexpr uint32 SequenceIndexBits = 20;
	constexpr uint32 SequenceIndexMask = MakeBitMask(SequenceIndexBits);

	constexpr uint32 LoopModeShift = SequenceIndexBits;
	constexpr uint32 LoopModeBits = 1;
	constexpr uint32 LoopModeMask = MakeBitMask(LoopModeBits);

	constexpr uint32 AutoPlayShift = LoopModeShift + LoopModeBits;
	constexpr uint32 AutoPlayBits = 1;
	constexpr uint32 AutoPlayMask = MakeBitMask(AutoPlayBits);

	constexpr uint32 BlendSpaceShift = AutoPlayShift + AutoPlayBits;
	constexpr uint32 BlendSpaceBits = 1;
	constexpr uint32 BlendSpaceMask = MakeBitMask(BlendSpaceBits);

	uint32 PackedBits;

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AnimSequenceTransformProviderLayers)
	{
		if (Ar.IsSaving())
		{
			PackedBits =
				(SequenceIndex & SequenceIndexMask)
				| ((LoopMode & LoopModeMask) << LoopModeShift)
				| ((bAutoPlay & AutoPlayMask) << AutoPlayShift)
				| ((bBlendSpace & BlendSpaceMask) << BlendSpaceShift);
		}

		Ar << PackedBits;

		if (Ar.IsLoading())
		{
			SequenceIndex = PackedBits & SequenceIndexMask;
			LoopMode = (PackedBits >> LoopModeShift) & LoopModeMask;
			bAutoPlay = (PackedBits >> AutoPlayShift) & AutoPlayMask;
			bBlendSpace = (PackedBits >> BlendSpaceShift) & BlendSpaceMask;
			PatchFlags = static_cast<uint32>(EAnimSequenceTrackPatchFlags::All);
		}

		if (Ar.IsLoading() && bBlendSpace)
		{
			FMemory::Memzero(&BlendSpace, sizeof(BlendSpace));
		}

		if (bBlendSpace)
		{
			Ar << BlendSpace.Position;
			Ar << BlendSpace.PlayRate.Encoded;
			Ar << BlendSpace.BlendPosition[0].Encoded;
			Ar << BlendSpace.BlendPosition[1].Encoded;
			Ar << BlendSpace.BlendSpaceIndex;
		}
		else if (bAutoPlay)
		{
			Ar << Auto.Position;
			Ar << Auto.PlayRate.Encoded;
		}
		else
		{
			Ar << Manual.Position;
		}

		Ar << BlendTime.Encoded;
		Ar << LayerWeight.Encoded;
	}
	else
	{
		Ar << PackedBits;
		SequenceIndex = PackedBits & SequenceIndexMask;
		LoopMode = (PackedBits >> LoopModeShift) & LoopModeMask;
		bAutoPlay = (PackedBits >> AutoPlayShift) & AutoPlayMask;
		bBlendSpace = 0;
		PatchFlags = static_cast<uint32>(EAnimSequenceTrackPatchFlags::All);

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AnimSequenceTrackTimestamps)
		{
			if (bAutoPlay)
			{
				Ar << Auto.Position;
				float LegacyPlayRate;
				Ar << LegacyPlayRate;
				Auto.PlayRate = FFloat16(LegacyPlayRate);
				float LegacyBlendTime;
				Ar << LegacyBlendTime;
				BlendTime = FFloat16(LegacyBlendTime);
			}
			else
			{
				Ar << Manual.Position;
				float LegacyPreviousPosition;
				Ar << LegacyPreviousPosition; // Discard
			}
		}
		else
		{
			float Position;
			Ar << Position;

			if (bAutoPlay)
			{
				Auto.Position = Position;
				float LegacyPlayRate;
				Ar << LegacyPlayRate;
				Auto.PlayRate = FFloat16(LegacyPlayRate);
				float LegacyBlendTime;
				Ar << LegacyBlendTime;
				BlendTime = FFloat16(LegacyBlendTime);
			}
			else
			{
				Manual.Position = Position;
				float LegacyPreviousPosition;
				Ar << LegacyPreviousPosition; // Discard
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FAnimSequenceTrackRenderData::Patch(const FAnimSequenceTrackPackedData& Data, double WorldTime, bool bWasActive)
{
	if (!bWasActive)
	{
		bHasPreviousPosition = false;
	}

	// Only patching specific flags.
	if (Data.GetPatchFlags() != EAnimSequenceTrackPatchFlags::None && Data.GetPatchFlags() != EAnimSequenceTrackPatchFlags::All)
	{
		if (EnumHasAnyFlags(Data.GetPatchFlags(), EAnimSequenceTrackPatchFlags::PlayRate))
		{
			Source.SetPlayRate(Data.GetPlayRate(), WorldTime);
			Target.SetPlayRate(Data.GetPlayRate(), WorldTime);
		}

		if (EnumHasAnyFlags(Data.GetPatchFlags(), EAnimSequenceTrackPatchFlags::LoopMode))
		{
			Source.SetLoopMode(Data.GetLoopMode());
			Target.SetLoopMode(Data.GetLoopMode());
		}
	}
	else
	{
		if (Data.GetBlendTime() <= 0.0f || !bWasActive || !CVarAnimSequenceTransformProviderBlending.GetValueOnRenderThread())
		{
			Source = Data;
		}
		else if (BlendWeight.GetFloat() > 0.0f)
		{
			Source = Target;
		}

		BlendWeight = FFloat16(0.0f);
		Target = Data;
	}

	Source.ResetPatchFlags();
	Target.ResetPatchFlags();
}

void FAnimSequenceTrackRenderData::Tick(float DeltaTime, double WorldTime)
{
	if (Target.IsBlendSpace())
	{
		const FVector2f BlendPos = Target.GetBlendSpacePosition();
		PreviousBlendSpacePosition[0] = FFloat16(BlendPos.X);
		PreviousBlendSpacePosition[1] = FFloat16(BlendPos.Y);
	}
	PreviousTimePosition = Target.GetTimePosition(WorldTime);
	PreviousBlendWeight = BlendWeight;
	bHasPreviousPosition = true;

	const float BlendTimeValue = Target.GetBlendTime();

	if (BlendTimeValue > 0.0f && BlendWeight.GetFloat() < 1.0f && CVarAnimSequenceTransformProviderBlending.GetValueOnRenderThread())
	{
		const float NewBlendWeight = BlendWeight.GetFloat() + DeltaTime / BlendTimeValue;

		if (NewBlendWeight >= 1.0f)
		{
			BlendWeight = FFloat16(0.0f);
			Source = Target;
		}
		else
		{
			BlendWeight = FFloat16(NewBlendWeight);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FAnimSequenceTransformProviderRenderData::~FAnimSequenceTransformProviderRenderData()
{
	check(Proxies.Num() == 0);
}

void FAnimSequenceTransformProviderRenderData::InsertProxy(FInstancedSkinningSceneExtensionProxy* Proxy)
{
	Proxies.FindOrAdd(Proxy);
}

void FAnimSequenceTransformProviderRenderData::RemoveProxy(FInstancedSkinningSceneExtensionProxy* Proxy)
{
	Proxies.Remove(Proxy);
}

void FAnimSequenceTransformProviderRenderData::Patch(TArray<FAnimSequenceTrackPool::FPatch>&& InPatches, TArray<FAnimSequenceTransformProviderRenderLayer>&& InLayers, double WorldTime)
{
	check(!InLayers.IsEmpty());
	check(InPatches.Num() == InLayers.Num());
	check(InLayers.Num() <= MAX_ANIM_SEQUENCE_LAYERS);
	Layers.SetNum(InLayers.Num());
	MeshSpaceMask = 0;
	LayerModeMask = 0;

	for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); LayerIndex++)
	{
		Layers[LayerIndex].Layer  = InLayers[LayerIndex].Layer;
		Layers[LayerIndex].Weight = InLayers[LayerIndex].Weight;

		if (Layers[LayerIndex].Layer.bMeshSpaceRotation)
		{
			MeshSpaceMask |= (1u << LayerIndex);
		}

		const uint32 Mode = Layers[LayerIndex].Layer.BlendMode == EAnimSequenceTransformProviderLayerBlendMode::Additive ? SKINNING_LAYER_MODE_ADDITIVE : SKINNING_LAYER_MODE_OVERRIDE;
		LayerModeMask |= (Mode << (LayerIndex * SKINNING_LAYER_MODE_BITS_PER_LAYER));
	}

	if (InPatches[0].GetNumTracks() != Layers[0].Tracks.GetNumTracks())
	{
		for (FInstancedSkinningSceneExtensionProxy* Proxy : Proxies)
		{
			Proxy->SetUniqueAnimationCount(InPatches[0].GetNumTracks());
		}
	}

	for (int32 LayerIndex = 0; LayerIndex < InPatches.Num(); LayerIndex++)
	{
		if (InPatches[LayerIndex].GetNumTracks() > 0)
		{
			Layers[LayerIndex].Tracks.Patch<FAnimSequenceTrackPackedData>(MoveTemp(InPatches[LayerIndex]), [WorldTime](FAnimSequenceTrackRenderData& Dst, const FAnimSequenceTrackPackedData& Src, bool bWasActive, int32 Index)
			{
				Dst.Patch(Src, WorldTime, bWasActive);
			});
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FAnimSequenceTransformProviderProxy::FAnimSequenceTransformProviderProxy(TConstArrayView<FAnimSequenceTransformProviderSequence> InSequences, FInstancedSkinningSceneExtensionProxy* InSceneExtensionProxy)
	: Sequences(InSequences)
	, RenderData(new FAnimSequenceTransformProviderRenderData())
	, SceneExtensionProxy(InSceneExtensionProxy)
	, bOwnsRenderData(true)
{
	FAnimSequenceTransformProviderRenderLayer& BaseLayer = RenderData->Layers.Emplace_GetRef();
	BaseLayer.Tracks.Reserve(Sequences.Num());

	for (int32 SequenceIndex = 0; SequenceIndex < Sequences.Num(); ++SequenceIndex)
	{
		const FAnimSequenceTransformProviderSequence& Sequence = Sequences[SequenceIndex];

		FAnimSequenceTrackAutoPlayData Data;
		Data.SequenceIndex = SequenceIndex;
		Data.Position = Sequence.Position;
		Data.PlayRate = Sequence.PlayRate;

		BaseLayer.Tracks.AllocateTrack(FAnimSequenceTrackPackedData::Pack(Data, 0.0));
	}
}

FAnimSequenceTransformProviderProxy::FAnimSequenceTransformProviderProxy(TConstArrayView<FAnimSequenceTransformProviderSequence> InSequences, FInstancedSkinningSceneExtensionProxy* InSceneExtensionProxy, FAnimSequenceTransformProviderRenderData& InRenderData)
	: Sequences(InSequences)
	, RenderData(&InRenderData)
	, SceneExtensionProxy(InSceneExtensionProxy)
{
}

FAnimSequenceTransformProviderProxy::~FAnimSequenceTransformProviderProxy()
{
	if (bOwnsRenderData)
	{
		delete RenderData;
	}
}

void FAnimSequenceTransformProviderProxy::CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList)
{
	if (RenderData)
	{
		RenderData->InsertProxy(SceneExtensionProxy);
	}

	AnimSequenceTransformProvider = Scene.GetAnimSequenceTransformProvider();

	if (AnimSequenceTransformProvider)
	{
		AnimSequenceTransformProvider->RegisterProxy(this);
	}
}

void FAnimSequenceTransformProviderProxy::DestroyRenderThreadResources()
{
	if (RenderData)
	{
		RenderData->RemoveProxy(SceneExtensionProxy);
	}

	if (AnimSequenceTransformProvider)
	{
		AnimSequenceTransformProvider->UnregisterProxy(this);
		AnimSequenceTransformProvider = nullptr;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

bool UAnimSequenceTransformProviderData::IsEnabled() const
{
	if (Sequences.IsEmpty())
	{
		return false;
	}

	for (const FAnimSequenceTransformProviderSequence& Sequence : Sequences)
	{
		if (Sequence.Sequence == nullptr)
		{
			return false;
		}
	}

	return true;
}

const FGuid& UAnimSequenceTransformProviderData::GetTransformProviderID() const
{
	static FGuid AnimSequenceProviderId(ANIM_SEQUENCE_GPU_TRANSFORM_PROVIDER_GUID);
	return AnimSequenceProviderId;
}

uint32 UAnimSequenceTransformProviderData::GetUniqueAnimationCount() const
{
	return Sequences.Num();
}

bool UAnimSequenceTransformProviderData::UsesSkeletonBatching() const
{
	return false;
}

bool UAnimSequenceTransformProviderData::HasAnimationBounds() const
{
	return CachedData.ConservativeBounds.SphereRadius > 0.0f;
}

bool UAnimSequenceTransformProviderData::GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const
{
	if (CachedData.ConservativeBounds.SphereRadius > 0.0f)
	{
		OutBounds = CachedData.ConservativeBounds;
		return true;
	}
	return false;
}

bool UAnimSequenceTransformProviderData::IsCompiling() const
{
#if WITH_EDITOR
	return CacheTasksByKeyHash.Num() > 0;
#else
	return false;
#endif
}

uint32 UAnimSequenceTransformProviderData::GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const
{
	const uint32 AnimationIndex = InstanceData.AnimationIndex;
	if (AnimationIndex < uint32(Sequences.Num()))
	{
		return AnimationIndex * 2u;
	}

	return 0u;
}

void UAnimSequenceTransformProviderData::BeginDestroy()
{
	DestroyFence.BeginFence();
	Super::BeginDestroy();
}

bool UAnimSequenceTransformProviderData::IsReadyForFinishDestroy()
{
	if (!Super::IsReadyForFinishDestroy())
	{
		return false;
	}

#if WITH_EDITOR
	if (!TryCancelAsyncTasks())
	{
		return false;
	}
#endif

	return DestroyFence.IsFenceComplete();
}

bool UAnimSequenceTransformProviderData::IsValidFor(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const
{
	if (!SkinnedAsset || !ExtensionProxy->GetSkinnedAsset())
	{
		UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: null skinned asset for provider '%ls'", *GetPathName());
		return false;
	}

	if (SkinnedAsset != ExtensionProxy->GetSkinnedAsset())
	{
		UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: SkinnedAsset '%ls' in provider '%ls' doesn't match SkinnedAsset on InstancedSkinnedMesh: '%ls'",
			*SkinnedAsset->GetPathName(), *GetPathName(), *ExtensionProxy->GetSkinnedAsset()->GetPathName());
		return false;
	}

	const FSkeletalMeshRenderData* RenderData = SkinnedAsset->GetResourceForRendering();
	if (!RenderData || !RenderData->HasUnifiedBoneMap())
	{
		UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: SkinnedAsset '%ls' in provider '%ls' requires 'Optimize for Instancing' enabled on the skeletal mesh in the build settings.",
			*SkinnedAsset->GetPathName(), *GetPathName());
		return false;
	}

	// Nanite skeletal meshes always render from clusters built against LOD 0's bone set.
	// ASTP must source bone metadata from LOD 0 to match those clusters — if LOD 0's render
	// data is unavailable (e.g., MinLodLevel > 0 stripped it out), we cannot register.
	const FSkeletalMeshObject* MeshObject = ExtensionProxy->GetMeshObject();
	if (MeshObject && MeshObject->IsNaniteMesh())
	{
		if (!RenderData->LODRenderData.IsValidIndex(0) || RenderData->LODRenderData[0].RequiredBones.IsEmpty())
		{
			UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: Nanite skinned asset '%ls' in provider '%ls' is missing LOD 0 render data. Nanite ASTP requires LOD 0 to be loaded (set MinLodLevel = 0).",
				*SkinnedAsset->GetPathName(), *GetPathName());
			return false;
		}
	}

	const bool bHasMeshSpaceLayer = Layers.ContainsByPredicate([](const FAnimSequenceTransformProviderLayer& Layer) { return Layer.bMeshSpaceRotation; });
	const int32 MaxBonesPerGroup = bHasMeshSpaceLayer ? SKINNING_MAX_BONES_PER_GROUP_MESHSPACE : SKINNING_MAX_BONES_PER_GROUP;
	const int32 MinLODLevel = ExtensionProxy->GetMeshObject() ? ExtensionProxy->GetMeshObject()->MinLODLevel : 0;
	for (int32 LODIndex = MinLODLevel; LODIndex < RenderData->LODRenderData.Num(); ++LODIndex)
	{
		if (RenderData->LODRenderData[LODIndex].RequiredBones.Num() > MaxBonesPerGroup)
		{
			UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: SkinnedAsset '%ls' in provider '%ls' LOD %d has %d bones which exceeds the maximum of %d%ls.",
				*SkinnedAsset->GetPathName(), *GetPathName(), LODIndex, RenderData->LODRenderData[LODIndex].RequiredBones.Num(), MaxBonesPerGroup,
				bHasMeshSpaceLayer ? TEXT(" (reduced due to mesh-space rotation layer)") : TEXT(""));
			return false;
		}
	}

	for (const FAnimSequenceTransformProviderSequence& Sequence : Sequences)
	{
		if (Sequence.Sequence == nullptr)
		{
			UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: Invalid null sequence in provider '%ls' for skinned asset '%ls'",
				*GetPathName(),
				*SkinnedAsset->GetPathName());
			return false;
		}

		const USkeleton* SourceSkeleton = Sequence.Sequence->GetSkeleton();
		const USkeleton* TargetSkeleton = SkinnedAsset->GetSkeleton();

		if (SourceSkeleton == nullptr)
		{
			UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: Sequence '%ls' has no skeleton, cannot use with skinned asset '%ls'",
				*Sequence.Sequence->GetPathName(),
				*SkinnedAsset->GetPathName());
			return false;
		}

		if (TargetSkeleton == nullptr)
		{
			UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: Skinned asset '%ls' has no skeleton, cannot use with sequence '%ls'",
				*SkinnedAsset->GetPathName(),
				*Sequence.Sequence->GetPathName());
			return false;
		}

		if (SourceSkeleton != TargetSkeleton)
		{
			const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);

			if (!SkeletonRemapping.IsValid())
			{
				UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: Incompatible skeletons: sequence '%ls' uses skeleton '%ls', but skinned asset '%ls' uses skeleton '%ls' with no valid remapping",
					*Sequence.Sequence->GetPathName(),
					*SourceSkeleton->GetPathName(),
					*SkinnedAsset->GetPathName(),
					*TargetSkeleton->GetPathName());
				return false;
			}
		}
	}

	for (int32 BlendSpaceIndex = 0; BlendSpaceIndex < BlendSpaces.Num(); BlendSpaceIndex++)
	{
		const UBlendSpace* BlendSpace = BlendSpaces[BlendSpaceIndex].BlendSpace;
		if (!BlendSpace)
		{
			UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: Null blend space at index %d in provider '%ls'",
				BlendSpaceIndex, *GetPathName());
			return false;
		}

		if (BlendSpace->GetBlendSamples().IsEmpty())
		{
			UE_LOGF(LogAnimation, Warning, "AnimSequenceTransformProvider: Blend space '%ls' at index %d in provider '%ls' has no samples.",
				*BlendSpace->GetPathName(), BlendSpaceIndex, *GetPathName());
			return false;
		}
	}

	return true;
}

FTransformProviderRenderProxy* UAnimSequenceTransformProviderData::CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy) const
{
	if (IsValidFor(SceneExtensionProxy))
	{
		return new FAnimSequenceTransformProviderProxy(Sequences, SceneExtensionProxy);
	}
	return nullptr;
}

float UAnimSequenceTransformProviderData::GetSequencePlayLength(int32 SequenceIndex) const
{
	if (Sequences.IsValidIndex(SequenceIndex) && Sequences[SequenceIndex].Sequence)
	{
		return Sequences[SequenceIndex].Sequence->GetPlayLength();
	}
	return 0.0f;
}

namespace AnimSequenceTrackTime
{
	float GetTimeScale()
	{
		static const auto* CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.AnimSequenceTransformProvider.TimeScale"));
		return CVar ? CVar->GetValueOnAnyThread() : 1.0f;
	}

	thread_local struct FTime
	{
		UWorld* World = nullptr;
		uint64 Frame = INDEX_NONE;
		double WorldTime = 0.0;

	} GTimeCache;

	double GetTime(UWorld* World)
	{
		if (World)
		{
			if (GTimeCache.Frame != GFrameCounter || GTimeCache.World != World)
			{
				GTimeCache = { World, GFrameCounter, World->GetTimeSeconds() * GetTimeScale() };
			}
			return GTimeCache.WorldTime;
		}
		return FGameTime::GetTimeSinceAppStart().GetWorldTimeSeconds();
	}
}

double UAnimSequenceTransformProviderData::GetWorldTime() const
{
#if WITH_ENGINE
	return AnimSequenceTrackTime::GetTime(GetWorld());
#else
	return 0.0;
#endif
}

UTransformProviderData* UAnimSequenceTransformProviderData::GetRootTransformProvider()
{
	return this;
}

#if WITH_EDITOR
void UAnimSequenceTransformProviderData::ForEachReferencingProvider(
	TFunctionRef<bool(const UAnimSequenceTransformProviderData*)> Predicate,
	TFunctionRef<void(UAnimSequenceTransformProviderData*)> Action)
{
	ForEachObjectOfClass(UAnimSequenceTransformProviderData::StaticClass(), [&Predicate, &Action](UObject* Obj)
	{
		UAnimSequenceTransformProviderData* Provider = CastChecked<UAnimSequenceTransformProviderData>(Obj);
		if (!Predicate(Provider))
		{
			return;
		}

		Action(Provider);

		if (UAnimSequenceTransformProviderDataInstance* Instance = Cast<UAnimSequenceTransformProviderDataInstance>(Provider))
		{
			Instance->InitLayerPools();
		}

		for (UInstancedSkinnedMeshComponent* Component : FObjectCacheContextScope().GetContext().GetInstancedSkinnedMeshComponents(Provider))
		{
			if (Component && IsValid(Component))
			{
				Component->MarkRenderStateDirty();
			}
		}
	});
}

void UAnimSequenceTransformProviderLayerStack::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (Layers.IsEmpty())
	{
		Layers.Emplace();
	}

	Layers[0] = FAnimSequenceTransformProviderLayer();

	if (Layers.Num() > MAX_ANIM_SEQUENCE_LAYERS)
	{
		Layers.SetNum(MAX_ANIM_SEQUENCE_LAYERS);
	}

	UAnimSequenceTransformProviderData::ForEachReferencingProvider(
		[this](const UAnimSequenceTransformProviderData* Provider) { return Provider->GetLayerStack() == this; },
		[this](UAnimSequenceTransformProviderData* Provider) { Provider->Layers = Layers; });
}

void UAnimSequenceTransformProviderSequenceList::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UAnimSequenceTransformProviderData::ForEachReferencingProvider(
		[this](const UAnimSequenceTransformProviderData* Provider) { return Provider->SequenceList == this; },
		[this](UAnimSequenceTransformProviderData* Provider) { Provider->Sequences = Sequences; });
}

void UAnimSequenceTransformProviderBlendSpaceList::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UAnimSequenceTransformProviderData::ForEachReferencingProvider(
		[this](const UAnimSequenceTransformProviderData* Provider) { return Provider->BlendSpaceList == this; },
		[this](UAnimSequenceTransformProviderData* Provider) { Provider->BlendSpaces = BlendSpaces; });
}
#endif

void UAnimSequenceTransformProviderData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Super::Serialize(Ar);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AnimSequenceTransformProviderBounds)
	{
		if (Ar.IsFilterEditorOnly() && !Ar.IsObjectReferenceCollector() && !Ar.ShouldSkipBulkData())
		{
	#if WITH_EDITOR
			if (Ar.IsCooking())
			{
				if (FAnimSequenceTransformProviderCachedData* DataPtr = CacheDerivedData(Ar.CookingTarget()))
				{
					Ar << *DataPtr;
				}
				else
				{
					FAnimSequenceTransformProviderCachedData EmptyData;
					Ar << EmptyData;
				}
			}
			else
	#endif
			{
				Ar << CachedData;
			}
		}
	}

	if (Ar.IsLoading())
	{
		if (Layers.IsEmpty())
		{
			Layers.Emplace();
		}
	}
}

void UAnimSequenceTransformProviderData::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Sync from shared assets to inline data. Ensure the source assets have completed
	// their own PostLoad before reading from them.
	if (SequenceList)
	{
		SequenceList->ConditionalPostLoad();
		Sequences = SequenceList->GetSequences();
	}

	if (LayerStack)
	{
		LayerStack->ConditionalPostLoad();
		Layers = LayerStack->GetLayers();
	}

	if (BlendSpaceList)
	{
		BlendSpaceList->ConditionalPostLoad();
		BlendSpaces = BlendSpaceList->GetBlendSpaces();
	}
#endif

#if WITH_EDITOR
	if (FApp::CanEverRender() && CachedData.SequenceBounds.IsEmpty())
	{
		BeginCacheDerivedData(GetTargetPlatformManagerRef().GetRunningTargetPlatform());
	}
#endif
}

#if WITH_EDITOR

void UAnimSequenceTransformProviderData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!IsTemplate() && !FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		MarkPackageDirty();
	}

	// Sync from shared assets to inline data
#if WITH_EDITORONLY_DATA
	if (SequenceList)
	{
		Sequences = SequenceList->GetSequences();
	}

	if (LayerStack)
	{
		Layers = LayerStack->GetLayers();
	}

	if (BlendSpaceList)
	{
		BlendSpaces = BlendSpaceList->GetBlendSpaces();
	}
#endif

	// Ensure layer 0 always exists
	if (Layers.IsEmpty())
	{
		Layers.Emplace();
	}

	// Ensure layer 0 is always the default.
	Layers[0] = FAnimSequenceTransformProviderLayer();

	// Ensure we stay under the maximum layer limit.
	if (Layers.Num() > MAX_ANIM_SEQUENCE_LAYERS)
	{
		Layers.SetNum(MAX_ANIM_SEQUENCE_LAYERS);
	}

	ForEachObjectOfClass(UAnimSequenceTransformProviderDataInstance::StaticClass(), [this](UObject* Obj)
	{
		if (UAnimSequenceTransformProviderDataInstance* Provider = Cast<UAnimSequenceTransformProviderDataInstance>(Obj))
		{
			if (Provider->GetRootTransformProvider() == this)
			{
				Provider->SetParentProvider(this);
			}
		}
	});

	BeginCacheDerivedData(GetTargetPlatformManagerRef().GetRunningTargetPlatform());

	// Mark render state dirty for components which reference this asset.
	FObjectCacheContextScope ObjectCacheScope;
	for (UInstancedSkinnedMeshComponent* Component : ObjectCacheScope.GetContext().GetInstancedSkinnedMeshComponents(this))
	{
		if (Component && IsValid(Component))
		{
			Component->MarkRenderStateDirty();
		}
	}
}

UAnimSequenceTransformProviderData* UAnimSequenceTransformProviderData::CreateFromAnimBankData(UAnimBankData* InAnimBankData)
{
	if (!InAnimBankData)
	{
		return nullptr;
	}

	UPackage* Package = InAnimBankData->GetOutermost();
	const FString NewAssetName = InAnimBankData->GetName() + TEXT("_ASTP");
	const FString NewAssetPath = FPackageName::GetLongPackagePath(Package->GetName()) / NewAssetName;

	if (StaticFindObject(UAnimSequenceTransformProviderData::StaticClass(), nullptr, *NewAssetPath))
	{
		UE_LOGF(LogAnimation, Warning, "CreateFromAnimBankData: Asset already exists at %ls", *NewAssetPath);
		return nullptr;
	}

	UPackage* NewPackage = CreatePackage(*NewAssetPath);
	if (!NewPackage)
	{
		return nullptr;
	}

	UAnimSequenceTransformProviderData* NewProviderData = NewObject<UAnimSequenceTransformProviderData>(NewPackage, *NewAssetName, RF_Public | RF_Standalone);

	if (!NewProviderData)
	{
		return nullptr;
	}

	for (const FAnimBankItem& BankItem : InAnimBankData->AnimBankItems)
	{
		FAnimSequenceTransformProviderSequence Sequence;
		if (BankItem.BankAsset)
		{
			if (!NewProviderData->SkinnedAsset)
			{
				NewProviderData->SkinnedAsset = BankItem.BankAsset->Asset;
			}

			const TArray<FAnimBankSequence>& BankSequences = BankItem.BankAsset->Sequences;
			if (BankSequences.IsValidIndex(BankItem.SequenceIndex))
			{
				const FAnimBankSequence& BankSequence = BankSequences[BankItem.SequenceIndex];
				Sequence.Sequence = BankSequence.Sequence;
				Sequence.PlayRate = BankSequence.PlayRate;
				Sequence.Position = BankSequence.Position;
			}
		}
		NewProviderData->Sequences.Add(Sequence);
	}

	NewProviderData->Layers.Emplace();
	NewProviderData->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(NewProviderData);
	return NewProviderData;
}

UAnimSequenceTransformProviderData* UAnimSequenceTransformProviderData::CreateFromAnimBank(UAnimBank* InAnimBank)
{
	if (!InAnimBank)
	{
		return nullptr;
	}

	UPackage* Package = InAnimBank->GetOutermost();
	const FString NewAssetName = InAnimBank->GetName() + TEXT("_ASTP");
	const FString NewAssetPath = FPackageName::GetLongPackagePath(Package->GetName()) / NewAssetName;

	if (StaticFindObject(UAnimSequenceTransformProviderData::StaticClass(), nullptr, *NewAssetPath))
	{
		UE_LOGF(LogAnimation, Warning, "CreateFromAnimBankData: Asset already exists at %ls", *NewAssetPath);
		return nullptr;
	}

	UPackage* NewPackage = CreatePackage(*NewAssetPath);
	if (!NewPackage)
	{
		return nullptr;
	}

	UAnimSequenceTransformProviderData* NewProviderData = NewObject<UAnimSequenceTransformProviderData>(NewPackage, *NewAssetName, RF_Public | RF_Standalone);

	if (!NewProviderData)
	{
		return nullptr;
	}

	for (const FAnimBankSequence& BankSequence : InAnimBank->Sequences)
	{
		FAnimSequenceTransformProviderSequence Sequence;
		Sequence.Sequence = BankSequence.Sequence;
		Sequence.PlayRate = BankSequence.PlayRate;
		Sequence.Position = BankSequence.Position;
		NewProviderData->Sequences.Add(Sequence);
	}

	NewProviderData->SkinnedAsset = InAnimBank->Asset;
	NewProviderData->Layers.Emplace();
	NewProviderData->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(NewProviderData);
	return NewProviderData;
}

EDataValidationResult UAnimSequenceTransformProviderData::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	if (SkinnedAsset)
	{
		const FSkeletalMeshRenderData* RenderData = SkinnedAsset->GetResourceForRendering();
		if (RenderData)
		{
			if (!RenderData->HasUnifiedBoneMap())
			{
				Context.AddError(FText::Format(LOCTEXT("NoUnifiedBoneMap",
					"AnimSequenceTransformProvider {0} requires 'Optimize for Instancing' on skinned asset {1}. It can be enabled in the skeletal mesh build settings."),
					FText::FromString(GetPathName()),
					FText::FromString(SkinnedAsset->GetPathName())
				));
				Result = EDataValidationResult::Invalid;
			}

			const bool bHasMeshSpaceLayer = Layers.ContainsByPredicate([](const FAnimSequenceTransformProviderLayer& Layer) { return Layer.bMeshSpaceRotation; });
			const int32 MaxBonesPerGroup = bHasMeshSpaceLayer ? SKINNING_MAX_BONES_PER_GROUP_MESHSPACE : SKINNING_MAX_BONES_PER_GROUP;
			for (const FSkeletalMeshLODRenderData& LODData : RenderData->LODRenderData)
			{
				if (LODData.RequiredBones.Num() > MaxBonesPerGroup)
				{
					Context.AddWarning(FText::Format(LOCTEXT("TooManyBones",
						"AnimSequenceTransformProvider {0} skinned asset {1} LOD has {2} bones which exceeds the maximum of {3}.{4}"),
						FText::FromString(GetPathName()),
						FText::FromString(SkinnedAsset->GetPathName()),
						FText::AsNumber(LODData.RequiredBones.Num()),
						FText::AsNumber(MaxBonesPerGroup),
						bHasMeshSpaceLayer ? LOCTEXT("MeshSpaceNote", " Limit is reduced due to mesh-space rotation layer.") : FText::GetEmpty()
					));
					break;
				}
			}
		}
	}

	for (const FAnimSequenceTransformProviderSequence& Sequence : Sequences)
	{
		if (Sequence.Sequence == nullptr)
		{
			Context.AddError(FText::Format(LOCTEXT("NullSequence",
				"AnimSequenceTransformProvider {0} invalid null sequence."),
				FText::FromString(GetPathName())
			));
			Result = EDataValidationResult::Invalid;
			continue;
		}

		const FFrameRate SampleRate = Sequence.Sequence->GetSamplingFrameRate();
		const int32 NumFrames = SampleRate.AsFrameTime(Sequence.Sequence->GetPlayLength()).RoundToFrame().Value;
		if (NumFrames > SKINNING_MAX_KEY_INDEX)
		{
			Context.AddWarning(FText::Format(LOCTEXT("TooManyFrames",
				"AnimSequenceTransformProvider {0} sequence {1} has {2} frames which exceeds the maximum of {3}. Frames will be clamped."),
				FText::FromString(GetPathName()),
				FText::FromString(Sequence.Sequence->GetPathName()),
				FText::AsNumber(NumFrames),
				FText::AsNumber(SKINNING_MAX_KEY_INDEX)
			));
		}

		const USkeleton* SourceSkeleton = Sequence.Sequence->GetSkeleton();

		if (SourceSkeleton == nullptr)
		{
			Context.AddError(FText::Format(LOCTEXT("NullSourceSkeleton",
				"AnimSequenceTransformProvider {0} sequence {1} has a null skeleton."),
				FText::FromString(GetPathName()),
				FText::FromString(Sequence.Sequence->GetPathName())
			));
			Result = EDataValidationResult::Invalid;
			continue;
		}

		if (SkinnedAsset)
		{
			const USkeleton* TargetSkeleton = SkinnedAsset->GetSkeleton();

			if (TargetSkeleton == nullptr)
			{
				Context.AddError(FText::Format(LOCTEXT("NullTargetSkeleton",
					"AnimSequenceTransformProvider {0} skinned mesh {1} has a null skeleton."),
					FText::FromString(GetPathName()),
					FText::FromString(SkinnedAsset->GetPathName())
				));
				Result = EDataValidationResult::Invalid;
				continue;
			}

			if (SourceSkeleton != TargetSkeleton)
			{
				const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);

				if (!SkeletonRemapping.IsValid())
				{
					Context.AddError(FText::Format(LOCTEXT("IncompatibleSkeletons",
						"AnimSequenceTransformProvider {0} Incompatible skeletons: sequence {1} uses skeleton {2}, but skinned asset {3} uses skeleton {4} with no valid remapping."),
						FText::FromString(GetPathName()),
						FText::FromString(Sequence.Sequence->GetPathName()),
						FText::FromString(SourceSkeleton->GetPathName()),
						FText::FromString(SkinnedAsset->GetPathName())
					));
					Result = EDataValidationResult::Invalid;
					continue;
				}
			}
		}
	}

	for (int32 BlendSpaceIndex = 0; BlendSpaceIndex < BlendSpaces.Num(); BlendSpaceIndex++)
	{
		const UBlendSpace* BlendSpace = BlendSpaces[BlendSpaceIndex].BlendSpace;
		if (!BlendSpace)
		{
			Context.AddError(FText::Format(LOCTEXT("NullBlendSpace",
				"AnimSequenceTransformProvider {0} has a null blend space at index {1}."),
				FText::FromString(GetPathName()),
				FText::AsNumber(BlendSpaceIndex)
			));
			Result = EDataValidationResult::Invalid;
			continue;
		}

		const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
		if (Samples.IsEmpty())
		{
			Context.AddError(FText::Format(LOCTEXT("EmptyBlendSpace",
				"AnimSequenceTransformProvider {0} blend space {1} at index {2} has no samples."),
				FText::FromString(GetPathName()),
				FText::FromString(BlendSpace->GetPathName()),
				FText::AsNumber(BlendSpaceIndex)
			));
			Result = EDataValidationResult::Invalid;
			continue;
		}

		for (const FBlendSample& Sample : Samples)
		{
			const UAnimSequence* SampleSequence = Cast<UAnimSequence>(Sample.Animation);
			if (!SampleSequence)
			{
				continue;
			}

			bool bFound = false;
			for (const FAnimSequenceTransformProviderSequence& Seq : Sequences)
			{
				if (Seq.Sequence == SampleSequence)
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				Context.AddError(FText::Format(LOCTEXT("BlendSpaceSampleNotInSequences",
					"AnimSequenceTransformProvider {0} blend space {1} sample sequence {2} is not in the Sequences array."),
					FText::FromString(GetPathName()),
					FText::FromString(BlendSpace->GetPathName()),
					FText::FromString(SampleSequence->GetPathName())
				));
				Result = EDataValidationResult::Invalid;
			}
		}
	}

	return Result;
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////////////////////////

class UAnimSequenceTransformProviderDataInstance::FPlaybackTracker
{
public:
	explicit FPlaybackTracker(int32 InNumLayers)
		: NumLayers(InNumLayers)
	{}

	void SetSourceData(int32 TrackIndex, int32 LayerIndex, const FAnimSequenceTrackPackedData& Current, float InBlendTime, double WorldTime)
	{
		const int32 RequiredSize = (TrackIndex + 1) * NumLayers;
		if (States.Num() < RequiredSize)
		{
			NumTracks = TrackIndex + 1;
			States.SetNum(RequiredSize);
		}

		FBlendState& State = GetState(TrackIndex, LayerIndex);

		if (State.BlendStartTime < 0.0)
		{
			State.BlendStartTime = 0.0;
			return;
		}

		if (InBlendTime > 0.0f)
		{
			if (Current.IsManual())
			{
				State.ManualPosition = Current.GetManualPosition();
			}
			else
			{
				State.ReferenceTimestamp = Current.GetReferenceTimestamp();
			}

			State.PlayRate = FFloat16(Current.GetPlayRate());
			State.SequenceIndex = Current.IsBlendSpace() ? 0 : Current.GetSequenceIndex();
			State.LoopMode = static_cast<uint32>(Current.GetLoopMode());
			State.bAutoPlay = Current.IsAutoPlay();
			State.bBlendSpace = Current.IsBlendSpace();
			State.BlendStartTime = WorldTime;
			State.BlendTime = FFloat16(InBlendTime);
		}
	}

	FAnimSequenceTrackPosition GetPosition(
		int32 TrackIndex,
		int32 LayerIndex,
		const FAnimSequenceTrackPackedData& CurrentTarget,
		double WorldTime,
		TConstArrayView<FAnimSequenceTransformProviderSequence> InSequences) const
	{
		FAnimSequenceTrackPosition Result;
		Result.TargetPosition = DerivePosition(CurrentTarget, WorldTime, InSequences);

		if (!IsValidIndex(TrackIndex, LayerIndex))
		{
			Result.SourcePosition = Result.TargetPosition;
			return Result;
		}

		const FBlendState& State = GetState(TrackIndex, LayerIndex);

		if (State.BlendStartTime >= 0.0)
		{
			const float BlendTimeValue = State.BlendTime.GetFloat();
			const float Elapsed = static_cast<float>(WorldTime - State.BlendStartTime);
			Result.BlendWeight = (BlendTimeValue > 0.0f) ? FMath::Clamp(Elapsed / BlendTimeValue, 0.0f, 1.0f) : 0.0f;
			Result.BlendTime = BlendTimeValue;

			if (Result.IsBlending())
			{
				Result.SourcePosition = DerivePosition(State, WorldTime, InSequences);
			}
			else
			{
				Result.SourcePosition = Result.TargetPosition;
				Result.BlendWeight = 0.0f;
			}
		}
		else
		{
			Result.SourcePosition = Result.TargetPosition;
		}

		return Result;
	}

private:
	struct FBlendState
	{
		uint32 SequenceIndex : 20 = 0;
		uint32 LoopMode      : 1  = 0;
		uint32 bAutoPlay     : 1  = 0;
		uint32 bBlendSpace   : 1  = 0;
		union
		{
			double ReferenceTimestamp;
			float ManualPosition;
		};
		double BlendStartTime     = -1.0;
		FFloat16 PlayRate         = FFloat16(1.0f);
		FFloat16 BlendTime        = FFloat16(0.0f);
	};

	FBlendState& GetState(int32 TrackIndex, int32 LayerIndex)
	{
		return States[TrackIndex * NumLayers + LayerIndex];
	}

	const FBlendState& GetState(int32 TrackIndex, int32 LayerIndex) const
	{
		return States[TrackIndex * NumLayers + LayerIndex];
	}

	bool IsValidIndex(int32 TrackIndex, int32 LayerIndex) const
	{
		const int32 Index = TrackIndex * NumLayers + LayerIndex;
		return Index >= 0 && Index < States.Num();
	}

	static float DerivePosition(const FAnimSequenceTrackPackedData& Data, double WorldTime, TConstArrayView<FAnimSequenceTransformProviderSequence> InSequences)
	{
		if (Data.IsBlendSpace())
		{
			return static_cast<float>(Data.GetTimePosition(WorldTime));
		}

		const int32 SequenceIndex = Data.GetSequenceIndex();

		if (!InSequences.IsValidIndex(SequenceIndex) || !InSequences[SequenceIndex].Sequence)
		{
			return 0.0f;
		}

		return FAnimSequenceTrackPackedData::WrapPosition(
			Data.GetLoopMode(),
			Data.GetTimePosition(WorldTime),
			InSequences[SequenceIndex].Sequence->GetPlayLength());
	}

	float DerivePosition(const FBlendState& State, double WorldTime, TConstArrayView<FAnimSequenceTransformProviderSequence> InSequences) const
	{
		if (State.bAutoPlay || State.bBlendSpace)
		{
			const double RawPosition = (WorldTime - State.ReferenceTimestamp) * State.PlayRate.GetFloat();

			if (State.bBlendSpace)
			{
				// Blend space has no single sequence length, return raw time.
				return static_cast<float>(RawPosition);
			}

			if (InSequences.IsValidIndex(State.SequenceIndex) && InSequences[State.SequenceIndex].Sequence)
			{
				return FAnimSequenceTrackPackedData::WrapPosition(
					static_cast<EAnimSequenceTrackLoopMode>(State.LoopMode),
					RawPosition,
					InSequences[State.SequenceIndex].Sequence->GetPlayLength());
			}

			return static_cast<float>(RawPosition);
		}

		return State.ManualPosition;
	}

	TArray<FBlendState> States;
	int32 NumTracks = 0;
	int32 NumLayers = 0;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

UAnimSequenceTransformProviderDataInstance* UAnimSequenceTransformProviderDataInstance::CreateAnimSequenceTransformProviderDataInstance(UAnimSequenceTransformProviderData* ProviderData, UInstancedSkinnedMeshComponent* Owner)
{
	if (IsValid(ProviderData) && IsValid(Owner))
	{
		UAnimSequenceTransformProviderDataInstance* AnimSequenceInstance = NewObject<UAnimSequenceTransformProviderDataInstance>(Owner);
		AnimSequenceInstance->SetParentProvider(ProviderData);
		return AnimSequenceInstance;
	}

	return nullptr;
}

void UAnimSequenceTransformProviderDataInstance::SetParentProvider(UAnimSequenceTransformProviderData* ProviderData)
{
	check(ProviderData);
	ParentProvider = ProviderData;
	SkinnedAsset = ProviderData->SkinnedAsset;
	Sequences = ProviderData->Sequences;
	Layers = ProviderData->Layers;
	BlendSpaces = ProviderData->BlendSpaces;
	bEnableBlendTracking = ProviderData->bEnableBlendTracking;

#if WITH_EDITORONLY_DATA
	SequenceList = ProviderData->SequenceList;
	LayerStack = ProviderData->LayerStack;
	BlendSpaceList = ProviderData->BlendSpaceList;
#endif

	InitLayerPools();
}

bool UAnimSequenceTransformProviderDataInstance::IsEnabled() const
{
	return !TrackPools.IsEmpty() && TrackPools[0].GetNumTracks() != 0 && Super::IsEnabled();
}

uint32 UAnimSequenceTransformProviderDataInstance::GetUniqueAnimationCount() const
{
	return !TrackPools.IsEmpty() ? TrackPools[0].GetNumTracks() : 0;
}

uint32 UAnimSequenceTransformProviderDataInstance::GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const
{
	const uint32 AnimationIndex = InstanceData.AnimationIndex;
	if (!TrackPools.IsEmpty() && AnimationIndex < uint32(TrackPools[0].GetNumTracks()))
	{
		return AnimationIndex * 2u;
	}

	return 0u;
}

bool UAnimSequenceTransformProviderDataInstance::HasAnimationBounds() const
{
	return ParentProvider && ParentProvider->HasAnimationBounds();
}

bool UAnimSequenceTransformProviderDataInstance::GetAnimationBounds(uint32 TrackIndex, FRenderBounds& OutBounds) const
{
	if (!ParentProvider)
	{
		return false;
	}
	return ParentProvider->GetAnimationBounds(TrackIndex, OutBounds);
}

UTransformProviderData* UAnimSequenceTransformProviderDataInstance::GetRootTransformProvider()
{
	return ParentProvider;
}

bool UAnimSequenceTransformProviderDataInstance::IsCompiling() const
{
	if (ParentProvider && ParentProvider->IsCompiling())
	{
		return true;
	}

	return Super::IsCompiling();
}

FTransformProviderRenderProxy* UAnimSequenceTransformProviderDataInstance::CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy) const
{
	if (IsValidFor(SceneExtensionProxy))
	{
		return new FAnimSequenceTransformProviderProxy(Sequences, SceneExtensionProxy, *RenderData);
	}
	return nullptr;
}

void UAnimSequenceTransformProviderDataInstance::BeginDestroy()
{
	ENQUEUE_RENDER_COMMAND(AnimSequenceDataInstanceRelease)([RenderData = MoveTemp(RenderData)](FRHICommandList&) mutable
	{
		// Move destruction onto the render thread.
	});

	PlaybackTracker.Reset();

	Super::BeginDestroy();
}

void UAnimSequenceTransformProviderDataInstance::MarkOwnerRenderStateDirty()
{
	if (bPendingRenderStateDirty)
	{
		return;
	}

	if (!CachedOwnerComponent)
	{
		CachedOwnerComponent = GetTypedOuter<UInstancedSkinnedMeshComponent>();
	}

	if (CachedOwnerComponent)
	{
		CachedOwnerComponent->MarkRenderInstancesDirty();
		bPendingRenderStateDirty = true;
	}
}

void UAnimSequenceTransformProviderDataInstance::InitLayerPools(bool bMarkDirty)
{
	const int32 NumLayers = Layers.Num();
	check(NumLayers >= 1);

	TrackPools.Reserve(NumLayers);

	while (TrackPools.Num() < NumLayers)
	{
		FAnimSequenceTrackPool& Pool = TrackPools.Emplace_GetRef();

		if (TrackPools.Num() > 1)
		{
			Pool.InitFrom(TrackPools[0], [](int32) { return FAnimSequenceTrackPackedData{}; });
		}
	}

	while (TrackPools.Num() > NumLayers)
	{
		TrackPools.Pop();
	}

	bTrackPoolsDirty = true;
	BuildBlendSpaceCaches();

	// Create, recreate, or destroy playback tracker based on parent setting.
	if (bEnableBlendTracking)
	{
		PlaybackTracker = MakePimpl<FPlaybackTracker>(Layers.Num());
	}
	else
	{
		PlaybackTracker.Reset();
	}

	if (bMarkDirty)
	{
		MarkOwnerRenderStateDirty();
	}
}

bool UAnimSequenceTransformProviderDataInstance::IsValidTrackAndLayer(int32 TrackIndex, int32 LayerIndex) const
{
	return TrackPools.IsValidIndex(LayerIndex) && TrackPools[LayerIndex].IsActiveIndex(TrackIndex);
}

void UAnimSequenceTransformProviderDataInstance::BuildBlendSpaceCaches()
{
	BlendSpaceCaches.SetNum(BlendSpaces.Num());

	for (int32 BlendSpaceIndex = 0; BlendSpaceIndex < BlendSpaces.Num(); BlendSpaceIndex++)
	{
		FBlendSpaceCache& Cache = BlendSpaceCaches[BlendSpaceIndex];
		Cache.bValid = false;
		Cache.SampleToSequenceIndex.Reset();

		const UBlendSpace* BlendSpace = BlendSpaces[BlendSpaceIndex].BlendSpace;
		if (!BlendSpace)
		{
			continue;
		}

		const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
		Cache.SampleToSequenceIndex.SetNum(Samples.Num());

		int32 NumSamplesFound = 0;

		for (int32 SampleIndex = 0; SampleIndex < Samples.Num(); SampleIndex++)
		{
			const UAnimSequence* SampleSequence = Cast<UAnimSequence>(Samples[SampleIndex].Animation);
			Cache.SampleToSequenceIndex[SampleIndex] = INDEX_NONE;

			if (SampleSequence)
			{
				for (int32 SequenceIndex = 0; SequenceIndex < Sequences.Num(); SequenceIndex++)
				{
					if (Sequences[SequenceIndex].Sequence == SampleSequence)
					{
						Cache.SampleToSequenceIndex[SampleIndex] = SequenceIndex;
						NumSamplesFound++;
						break;
					}
				}
			}
		}

		Cache.bValid = (NumSamplesFound == Samples.Num());
	}
}

void UAnimSequenceTransformProviderDataInstance::ResolveBlendSpaceSamples(FAnimSequenceTrackPackedData& Data) const
{
	const uint8 BlendSpaceIndex = Data.GetBlendSpaceIndex();

	if (!BlendSpaceCaches.IsValidIndex(BlendSpaceIndex) || !BlendSpaceCaches[BlendSpaceIndex].bValid)
	{
		Data.SetNumSamples(0);
		return;
	}

	const FBlendSpaceCache& Cache = BlendSpaceCaches[BlendSpaceIndex];
	const UBlendSpace* BlendSpace = BlendSpaces.IsValidIndex(BlendSpaceIndex) ? BlendSpaces[BlendSpaceIndex].BlendSpace.Get() : nullptr;
	if (!BlendSpace)
	{
		Data.SetNumSamples(0);
		return;
	}

	const FVector2f Position = Data.GetBlendSpacePosition();
	TArray<FBlendSampleData> SampleResults;
	int32 TriangulationIndex = 0;
	BlendSpace->GetSamplesFromBlendInput(FVector(Position.X, Position.Y, 0.0f), SampleResults, TriangulationIndex, false);

	if (SampleResults.IsEmpty())
	{
		Data.SetNumSamples(0);
		return;
	}

	const int32 NumSamples = FMath::Min(SampleResults.Num(), 3);
	Data.SetNumSamples(NumSamples);

	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FBlendSampleData& Sample = SampleResults[SampleIndex];
		const int32 BlendSampleIndex = Sample.SampleDataIndex;

		uint16 SequenceIndex = 0;
		if (Cache.SampleToSequenceIndex.IsValidIndex(BlendSampleIndex))
		{
			SequenceIndex = Cache.SampleToSequenceIndex[BlendSampleIndex];
		}

		Data.SetSampleData(SampleIndex, SequenceIndex, FFloat16(Sample.GetClampedWeight()));
	}
}

void UAnimSequenceTransformProviderDataInstance::SubmitChanges()
{
	check(RenderData);

	// SubmitChanges is called during the ISKM instance data update. Guard against multiple concurrent updates.
	UE::TScopeLock Lock(SubmitChangesMutex);

	check(TrackPools.Num() == Layers.Num());

	if (bNeedsPostLoadInit)
	{
		const double WorldTime = GetWorldTime();

		for (FAnimSequenceTrackPool& Pool : TrackPools)
		{
			Pool.EnumerateActiveTracks([&](int32 TrackIndex)
			{
				Pool.GetData(TrackIndex).PostLoad(WorldTime);
				Pool.SetDirty(TrackIndex);
			});
		}

		bNeedsPostLoadInit = false;
	}

	bPendingRenderStateDirty = false;

	// Resolve blend space triangulation for dirty blend space tracks
	for (int32 LayerIndex = 0; LayerIndex < TrackPools.Num(); LayerIndex++)
	{
		FAnimSequenceTrackPool& Pool = TrackPools[LayerIndex];
		Pool.EnumerateActiveTracks([&](int32 TrackIndex)
		{
			if (Pool.IsDirtyIndex(TrackIndex))
			{
				FAnimSequenceTrackPackedData& Data = Pool.GetData(TrackIndex);
				if (Data.IsBlendSpace())
				{
					ResolveBlendSpaceSamples(Data);
				}
			}
		});
	}

	const int32 NumLayers = Layers.Num();
	bool bAnyDirty = bTrackPoolsDirty;
	for (const FAnimSequenceTrackPool& Pool : TrackPools)
	{
		bAnyDirty |= Pool.GetNumDirtyTracks() > 0;
	}

	if (bAnyDirty)
	{
		TArray<FAnimSequenceTrackPool::FPatch> Patches;
		Patches.SetNumZeroed(NumLayers);

		TArray<FAnimSequenceTransformProviderRenderLayer> PatchLayers;
		PatchLayers.SetNum(NumLayers);

		for (int32 LayerIndex = 0; LayerIndex < NumLayers; LayerIndex++)
		{
			Patches[LayerIndex] = TrackPools[LayerIndex].Finalize([](FAnimSequenceTrackPackedData& Data)
			{
				Data.ResetPatchFlags();
			});

			PatchLayers[LayerIndex].Layer  = Layers[LayerIndex];
			PatchLayers[LayerIndex].Weight = Layers[LayerIndex].Weight;
		}

		bTrackPoolsDirty = false;

		ENQUEUE_RENDER_COMMAND(AnimSequenceDataInstanceSubmit)(
			[
				  &RenderData  = *RenderData
				, Patches      = MoveTemp(Patches)
				, InLayers     = MoveTemp(PatchLayers)
				, WorldTime    = GetWorldTime()
			](FRHICommandList&) mutable
		{
			RenderData.Patch(MoveTemp(Patches), MoveTemp(InLayers), WorldTime);
		});
	}
}

void UAnimSequenceTransformProviderDataInstance::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Super::Serialize(Ar);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AnimSequenceTransformProviderLayers)
	{
		int32 NumLayers = TrackPools.Num();
		Ar << NumLayers;

		if (Ar.IsLoading())
		{
			TrackPools.SetNum(NumLayers);
		}

		for (int32 LayerIndex = 0; LayerIndex < NumLayers; LayerIndex++)
		{
			TrackPools[LayerIndex].Serialize(Ar);
		}
	}
	else
	{
		FAnimSequenceTrackPool LegacyPool;
		LegacyPool.Serialize(Ar);

		TrackPools.SetNum(1);
		TrackPools[0] = MoveTemp(LegacyPool);
	}

	if (Ar.IsLoading())
	{
		InitLayerPools(false);
		bNeedsPostLoadInit = true;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void UAnimSequenceTransformProviderDataInstance::SetLayerWeight(int32 TrackIndex, int32 LayerIndex, float Weight)
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return;
	}

	TrackPools[LayerIndex].GetData(TrackIndex).SetLayerWeight(FMath::Clamp(Weight, 0.0f, 1.0f));
	TrackPools[LayerIndex].SetDirty(TrackIndex);
	MarkOwnerRenderStateDirty();
}

float UAnimSequenceTransformProviderDataInstance::GetLayerWeight(int32 TrackIndex, int32 LayerIndex) const
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return 1.0f;
	}

	return TrackPools[LayerIndex].GetData(TrackIndex).GetLayerWeight();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

int32 UAnimSequenceTransformProviderDataInstance::AllocateTrack()
{
	check(!TrackPools.IsEmpty());
	FAnimSequenceTrackPackedData DefaultData{};
	const int32 TrackIndex = TrackPools[0].AllocateTrack(DefaultData);

	if (TrackIndex != INDEX_NONE)
	{
		for (int32 LayerIndex = 1; LayerIndex < TrackPools.Num(); LayerIndex++)
		{
			TrackPools[LayerIndex].AllocateTrackAt(TrackIndex, DefaultData);
		}
		MarkOwnerRenderStateDirty();
	}

	return TrackIndex;
}

bool UAnimSequenceTransformProviderDataInstance::DeallocateTrack(int32 TrackIndex)
{
	bool bDeallocated = false;

	for (FAnimSequenceTrackPool& Pool : TrackPools)
	{
		bDeallocated |= Pool.DeallocateTrack(TrackIndex);
	}

	if (bDeallocated)
	{
		MarkOwnerRenderStateDirty();
	}

	return bDeallocated;
}

bool UAnimSequenceTransformProviderDataInstance::SetAutoPlayData(int32 TrackIndex, int32 LayerIndex, const FAnimSequenceTrackAutoPlayData& Data)
{
	if (!TrackPools.IsValidIndex(LayerIndex) || !Sequences.IsValidIndex(Data.SequenceIndex))
	{
		return false;
	}

	FAnimSequenceTrackPool& Pool = TrackPools[LayerIndex];

	if (Pool.IsActiveIndex(TrackIndex))
	{
		if (PlaybackTracker)
		{
			PlaybackTracker->SetSourceData(TrackIndex, LayerIndex, Pool.GetData(TrackIndex), Data.BlendTime, GetWorldTime());
		}

		if (Pool.UpdateTrack(TrackIndex, FAnimSequenceTrackPackedData::Pack(Data, GetWorldTime())))
		{
			MarkOwnerRenderStateDirty();
			return true;
		}
	}

	return false;
}

bool UAnimSequenceTransformProviderDataInstance::SetManualData(int32 TrackIndex, int32 LayerIndex, const FAnimSequenceTrackManualData& Data)
{
	if (!TrackPools.IsValidIndex(LayerIndex) || !Sequences.IsValidIndex(Data.SequenceIndex))
	{
		return false;
	}

	FAnimSequenceTrackPool& Pool = TrackPools[LayerIndex];

	if (Pool.IsActiveIndex(TrackIndex))
	{
		if (PlaybackTracker)
		{
			PlaybackTracker->SetSourceData(TrackIndex, LayerIndex, Pool.GetData(TrackIndex), Data.BlendTime, GetWorldTime());
		}

		if (Pool.UpdateTrack(TrackIndex, FAnimSequenceTrackPackedData::Pack(Data, GetSequencePlayLength(Data.SequenceIndex))))
		{
			MarkOwnerRenderStateDirty();
			return true;
		}
	}

	return false;
}

bool UAnimSequenceTransformProviderDataInstance::SetPlayRate(int32 TrackIndex, int32 LayerIndex, float PlayRate)
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return false;
	}

	FAnimSequenceTrackPackedData& Data = TrackPools[LayerIndex].GetData(TrackIndex);
	if (Data.IsAutoPlay() || Data.IsBlendSpace())
	{
		Data.SetPlayRate(PlayRate, GetWorldTime());
		TrackPools[LayerIndex].SetDirty(TrackIndex);
		MarkOwnerRenderStateDirty();
		return true;
	}

	return false;
}

float UAnimSequenceTransformProviderDataInstance::GetPlayRate(int32 TrackIndex, int32 LayerIndex) const
{
	if (IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return TrackPools[LayerIndex].GetData(TrackIndex).GetPlayRate();
	}

	return 1.0f;
}

bool UAnimSequenceTransformProviderDataInstance::SetLoopMode(int32 TrackIndex, int32 LayerIndex, EAnimSequenceTrackLoopMode LoopMode)
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return false;
	}

	TrackPools[LayerIndex].GetData(TrackIndex).SetLoopMode(LoopMode);
	TrackPools[LayerIndex].SetDirty(TrackIndex);
	MarkOwnerRenderStateDirty();
	return true;
}

EAnimSequenceTrackLoopMode UAnimSequenceTransformProviderDataInstance::GetLoopMode(int32 TrackIndex, int32 LayerIndex) const
{
	if (IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return TrackPools[LayerIndex].GetData(TrackIndex).GetLoopMode();
	}

	return EAnimSequenceTrackLoopMode::Loop;
}

int32 UAnimSequenceTransformProviderDataInstance::GetSequenceIndex(int32 TrackIndex, int32 LayerIndex) const
{
	if (IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return TrackPools[LayerIndex].GetData(TrackIndex).GetSequenceIndex();
	}

	return INDEX_NONE;
}

EAnimSequenceTrackMode UAnimSequenceTransformProviderDataInstance::GetTrackMode(int32 TrackIndex, int32 LayerIndex) const
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return EAnimSequenceTrackMode::Manual;
	}

	const FAnimSequenceTrackPackedData& Data = TrackPools[LayerIndex].GetData(TrackIndex);

	if (Data.IsBlendSpace())
	{
		return EAnimSequenceTrackMode::BlendSpace;
	}
	if (Data.IsAutoPlay())
	{
		return EAnimSequenceTrackMode::AutoPlay;
	}
	return EAnimSequenceTrackMode::Manual;
}

FAnimSequenceTrackPosition UAnimSequenceTransformProviderDataInstance::GetPosition(int32 TrackIndex, int32 LayerIndex) const
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return {};
	}

	if (PlaybackTracker)
	{
		return PlaybackTracker->GetPosition(TrackIndex, LayerIndex, TrackPools[LayerIndex].GetData(TrackIndex), GetWorldTime(), Sequences);
	}

	// No tracker. Return position with Source = Target and no blend info.
	FAnimSequenceTrackPosition Result;
	const FAnimSequenceTrackPackedData& Data = TrackPools[LayerIndex].GetData(TrackIndex);

	if (Data.IsBlendSpace())
	{
		// Blend space has no single sequence — return raw time.
		Result.TargetPosition = static_cast<float>(Data.GetTimePosition(GetWorldTime()));
	}
	else
	{
		const int32 SeqIndex = Data.GetSequenceIndex();
		if (SeqIndex != INDEX_NONE && Sequences.IsValidIndex(SeqIndex))
		{
			Result.TargetPosition = FAnimSequenceTrackPackedData::WrapPosition(
				Data.GetLoopMode(),
				Data.GetTimePosition(GetWorldTime()),
				GetSequencePlayLength(SeqIndex));
		}
	}

	Result.SourcePosition = Result.TargetPosition;
	return Result;
}

bool UAnimSequenceTransformProviderDataInstance::SetBlendSpaceData(int32 TrackIndex, int32 LayerIndex, const FAnimSequenceTrackBlendSpaceData& InData)
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return false;
	}

	const int32 BlendSpaceIndex = InData.BlendSpaceIndex;
	if (!BlendSpaces.IsValidIndex(BlendSpaceIndex) || !BlendSpaceCaches.IsValidIndex(BlendSpaceIndex) || !BlendSpaceCaches[BlendSpaceIndex].bValid || BlendSpaceCaches[BlendSpaceIndex].SampleToSequenceIndex.IsEmpty())
	{
		return false;
	}

	if (PlaybackTracker)
	{
		PlaybackTracker->SetSourceData(TrackIndex, LayerIndex, TrackPools[LayerIndex].GetData(TrackIndex), InData.BlendTime, GetWorldTime());
	}

	TrackPools[LayerIndex].UpdateTrack(TrackIndex, FAnimSequenceTrackPackedData::Pack(InData, GetWorldTime()));
	MarkOwnerRenderStateDirty();
	return true;
}

bool UAnimSequenceTransformProviderDataInstance::SetBlendSpacePosition(int32 TrackIndex, int32 LayerIndex, FVector2f Position)
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return false;
	}

	FAnimSequenceTrackPackedData& Data = TrackPools[LayerIndex].GetData(TrackIndex);
	if (!Data.IsBlendSpace())
	{
		return false;
	}

	Data.SetBlendSpacePosition(Position);
	TrackPools[LayerIndex].SetDirty(TrackIndex);
	MarkOwnerRenderStateDirty();
	return true;
}

bool UAnimSequenceTransformProviderDataInstance::SetManualPosition(int32 TrackIndex, int32 LayerIndex, float Position)
{
	if (!IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		return false;
	}

	FAnimSequenceTrackPackedData& Data = TrackPools[LayerIndex].GetData(TrackIndex);
	if (!Data.IsManual())
	{
		return false;
	}

	const int32 SeqIndex = Data.GetSequenceIndex();
	if (SeqIndex == INDEX_NONE || !Sequences.IsValidIndex(SeqIndex))
	{
		return false;
	}

	Data.SetManualPosition(FAnimSequenceTrackPackedData::WrapPosition(Data.GetLoopMode(), Position, GetSequencePlayLength(SeqIndex)));
	TrackPools[LayerIndex].SetDirty(TrackIndex);
	MarkOwnerRenderStateDirty();
	return true;
}

FVector2f UAnimSequenceTransformProviderDataInstance::GetBlendSpacePosition(int32 TrackIndex, int32 LayerIndex) const
{
	if (IsValidTrackAndLayer(TrackIndex, LayerIndex))
	{
		const FAnimSequenceTrackPackedData& PackedData = TrackPools[LayerIndex].GetData(TrackIndex);
		if (PackedData.IsBlendSpace())
		{
			return PackedData.GetBlendSpacePosition();
		}
	}

	return FVector2f::ZeroVector;
}

#undef LOCTEXT_NAMESPACE