// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSet.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/UObjectIterator.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSet)

/////////////////////////////////////////////////////
// UAnimSet

UAnimSet::UAnimSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UAnimSet::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Make sure that AnimSets (and sequences) within level packages are not marked as standalone.
	if(GetOutermost()->ContainsMap() && HasAnyFlags(RF_Standalone))
	{
		ClearFlags(RF_Standalone);

		for(int32 i=0; i<Sequences.Num(); i++)
		{
			UAnimSequence* Seq = Sequences[i];
			if(Seq)
			{
				Seq->ClearFlags(RF_Standalone);
			}
		}
	}
#endif	//#if WITH_EDITORONLY_DATA
}	


bool UAnimSet::CanPlayOnSkeletalMesh(USkeletalMesh* SkelMesh) const
{
	// Temporarily allow any animation to play on any AnimSet. 
	// We need a looser metric for matching animation to skeletons. Some 'overlap bone count'?
#if 0
	// This is broken and needs to be looked into.
	// we require at least 10% of tracks matched by skeletal mesh.
	return GetSkeletalMeshMatchRatio(SkelMesh) > 0.1f;
#else
	return true;
#endif
}

float UAnimSet::GetSkeletalMeshMatchRatio(USkeletalMesh* SkelMesh) const
{
	// First see if there is a bone for all tracks
	int32 TracksMatched = 0;
	for(int32 i=0; i<TrackBoneNames.Num() ; i++)
	{
		const int32 BoneIndex = SkelMesh->GetRefSkeleton().FindBoneIndex( TrackBoneNames[i] );
		if( BoneIndex != INDEX_NONE )
		{
			++TracksMatched;
		}
	}

	// If we can't match any bones, then this is definitely not compatible.
	if( TrackBoneNames.Num() == 0 || TracksMatched == 0 )
	{
		return 0.f;
	}

	// return how many of the animation tracks were matched by that mesh
	return (float)TracksMatched / float(TrackBoneNames.Num());
}


UAnimSequence* UAnimSet::FindAnimSequence(FName SequenceName)
{
	UAnimSequence* AnimSequence = NULL;
#if WITH_EDITORONLY_DATA
	if( SequenceName != NAME_None )
	{
		for(int32 i=0; i<Sequences.Num(); i++)
		{
			if( Sequences[i]->GetFName() == SequenceName )
			{				
				AnimSequence = Sequences[i];
				break;
			}
		}
	}
#endif	//#if WITH_EDITORONLY_DATA

	return AnimSequence;
}


int32 UAnimSet::GetMeshLinkupIndex(USkeletalMesh* SkelMesh)
{
	// First, see if we have a cached link-up between this animation and the given skeletal mesh.
	check(SkelMesh);

	// Get SkeletalMesh path name
	FName SkelMeshName = FName( *SkelMesh->GetPathName() );

	// See if we have already cached this Skeletal Mesh.
	const int32* IndexPtr = SkelMesh2LinkupCache.Find( SkelMeshName );

	// If not found, create a new entry
	if( IndexPtr == NULL )
	{
		// No linkup found - so create one here and add to cache.
		const int32 NewLinkupIndex = LinkupCache.AddZeroed();

		// Add it to our cache
		SkelMesh2LinkupCache.Add( SkelMeshName, NewLinkupIndex );
		
		// Fill it up
		FAnimSetMeshLinkup* NewLinkup = &LinkupCache[NewLinkupIndex];
		NewLinkup->BuildLinkup(SkelMesh, this);

		return NewLinkupIndex;
	}

	return (*IndexPtr);
}

void UAnimSet::ResetAnimSet()
{
#if WITH_EDITORONLY_DATA
	// Make sure we handle AnimSequence references properly before emptying the array.
	for(int32 i=0; i<Sequences.Num(); i++)
	{	
		UAnimSequence* AnimSeq = Sequences[i];
		if( AnimSeq )
		{
			AnimSeq->ResetAnimation();
		}
	}
	Sequences.Empty();
	TrackBoneNames.Empty();
	LinkupCache.Empty();
	SkelMesh2LinkupCache.Empty();

	// We need to re-init any skeleltal mesh components now, because they might still have references to linkups in this set.
	for(TObjectIterator<USkeletalMeshComponent> It;It;++It)
	{
		USkeletalMeshComponent* SkelComp = *It;
		if(IsValid(SkelComp) && !SkelComp->IsTemplate())
		{
			SkelComp->InitAnim(true);
		}
	}
#endif // WITH_EDITORONLY_DATA
}


bool UAnimSet::RemoveAnimSequenceFromAnimSet(UAnimSequence* AnimSeq)
{
#if WITH_EDITORONLY_DATA
	int32 SequenceIndex = Sequences.Find(AnimSeq);
	if( SequenceIndex != INDEX_NONE )
	{
		// Handle reference clean up properly
		AnimSeq->ResetAnimation();
		// Remove from array
		Sequences.RemoveAt(SequenceIndex, 1);
		if( GIsEditor )
		{
			MarkPackageDirty();
		}
		return true;
	}
#endif // WITH_EDITORONLY_DATA

	return false;
}

void UAnimSet::ClearAllAnimSetLinkupCaches()
{
	double Start = FPlatformTime::Seconds();

	TArray<UAnimSet*> AnimSets;
	TArray<USkeletalMeshComponent*> SkelComps;
	// Find all AnimSets and SkeletalMeshComponents (just do one iterator)
	for(TObjectIterator<UObject> It;It;++It)
	{
		UAnimSet* AnimSet = Cast<UAnimSet>(*It);
		if(IsValid(AnimSet) && !AnimSet->IsTemplate())
		{
			AnimSets.Add(AnimSet);
		}

		USkeletalMeshComponent* SkelComp = Cast<USkeletalMeshComponent>(*It);
		if(IsValid(SkelComp) && !SkelComp->IsTemplate())
		{
			SkelComps.Add(SkelComp);
		}
	}

	// For all AnimSets, empty their linkup cache
	for(int32 i=0; i<AnimSets.Num(); i++)
	{
		AnimSets[i]->LinkupCache.Empty();
		AnimSets[i]->SkelMesh2LinkupCache.Empty();
	}

	UE_LOGF(LogAnimation, Log, "ClearAllAnimSetLinkupCaches - Took %3.2fms", (FPlatformTime::Seconds() - Start)*1000.f);
}


/////////////////////////////////////////////////////
// FAnimSetMeshLinkup

void FAnimSetMeshLinkup::BuildLinkup(USkeletalMesh* InSkelMesh, UAnimSet* InAnimSet)
{
	int32 const NumBones = InSkelMesh->GetRefSkeleton().GetNum();

	// Bone to Track mapping.
	BoneToTrackTable.Empty(NumBones);
	BoneToTrackTable.AddUninitialized(NumBones);

	// For each bone in skeletal mesh, find which track to pull from in the AnimSet.
	for (int32 i=0; i<NumBones; i++)
	{
		FName const BoneName = InSkelMesh->GetRefSkeleton().GetBoneName(i);

		// FindTrackWithName will return INDEX_NONE if no track exists.
		BoneToTrackTable[i] = InAnimSet->FindTrackWithName(BoneName);
	}

	// Check here if we've properly cached those arrays.
	if( InAnimSet->BoneUseAnimTranslation.Num() != InAnimSet->TrackBoneNames.Num() )
	{
		int32 const NumTracks = InAnimSet->TrackBoneNames.Num();

		InAnimSet->BoneUseAnimTranslation.Empty(NumTracks);
		InAnimSet->BoneUseAnimTranslation.AddUninitialized(NumTracks);

		InAnimSet->ForceUseMeshTranslation.Empty(NumTracks);
		InAnimSet->ForceUseMeshTranslation.AddUninitialized(NumTracks);

		for(int32 TrackIndex = 0; TrackIndex<NumTracks; TrackIndex++)
		{
			FName const TrackBoneName = InAnimSet->TrackBoneNames[TrackIndex];

			// Cache whether to use the translation from this bone or from ref pose.
			InAnimSet->BoneUseAnimTranslation[TrackIndex] = InAnimSet->UseTranslationBoneNames.Contains(TrackBoneName);
			InAnimSet->ForceUseMeshTranslation[TrackIndex] = InAnimSet->ForceMeshTranslationBoneNames.Contains(TrackBoneName);
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TArray<bool> TrackUsed;
	TrackUsed.AddZeroed(InAnimSet->TrackBoneNames.Num());
	const int32 AnimLinkupIndex = InAnimSet->GetMeshLinkupIndex( InSkelMesh );
	const FAnimSetMeshLinkup& AnimLinkup = InAnimSet->LinkupCache[ AnimLinkupIndex ];
	for(int32 BoneIndex=0; BoneIndex<NumBones; BoneIndex++)
	{
		const int32 TrackIndex = AnimLinkup.BoneToTrackTable[BoneIndex];

		if( TrackIndex == INDEX_NONE )
		{
			continue;
		}

		if( TrackUsed[TrackIndex] )
		{
			UE_LOGF(LogAnimation, Warning, "%ls has multiple bones sharing the same track index!!!", *InAnimSet->GetFullName());	
			for(int32 DupeBoneIndex=0; DupeBoneIndex<NumBones; DupeBoneIndex++)
			{
				const int32 DupeTrackIndex = AnimLinkup.BoneToTrackTable[DupeBoneIndex];
				if( DupeTrackIndex == TrackIndex )
				{
					UE_LOGF(LogAnimation, Warning, " BoneIndex: %i, BoneName: %ls, TrackIndex: %i, TrackBoneName: %ls", DupeBoneIndex, *InSkelMesh->GetRefSkeleton().GetBoneName(DupeBoneIndex).ToString(), DupeTrackIndex, *InAnimSet->TrackBoneNames[DupeTrackIndex].ToString());
				}
			}
		}

		TrackUsed[TrackIndex] = true;
	}
#endif
}


