// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UAnimSingleNodeInstance.cpp: Single Node Tree Instance 
	Only plays one animation at a time. 
=============================================================================*/ 

#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimSingleNodeInstanceProxy.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSingleNodeInstance)

/////////////////////////////////////////////////////
// UAnimSingleNodeInstance
/////////////////////////////////////////////////////

UAnimSingleNodeInstance::UAnimSingleNodeInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimSingleNodeInstance::SetAnimationAsset(class UAnimationAsset* NewAsset, bool bInIsLooping, float InPlayRate)
{
	if (NewAsset != CurrentAsset)
	{
		CurrentAsset = NewAsset;
	}

	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();

	if (
#if WITH_EDITOR
		!Proxy.CanProcessAdditiveAnimations() &&
#endif
		NewAsset && NewAsset->IsValidAdditive())
	{
		UE_LOGF(LogAnimation, Warning, "Setting an additive animation (%ls) on an AnimSingleNodeInstance (%ls) is not allowed. This will not function correctly in cooked builds!", *NewAsset->GetName(), *GetFullName());
	}

	USkeletalMeshComponent* MeshComponent = GetSkelMeshComponent();
	if (MeshComponent)
	{
		if (MeshComponent->GetSkeletalMeshAsset() == nullptr)
		{
			// if it does not have SkeletalMesh, we nullify it
			CurrentAsset = nullptr;
		}
		else if (CurrentAsset != nullptr)
		{
			// if we have an asset, make sure their skeleton is valid, otherwise, null it
			if (CurrentAsset->GetSkeleton() == nullptr)
			{
				// clear asset since we do not have matching skeleton
				CurrentAsset = nullptr;
			}
		}
		
		// We've changed the animation asset, and the next frame could be wildly different from the frame we're
		// on now. In this case of a single node instance, we reset the clothing on the next update.
		MeshComponent->ClothTeleportMode = EClothingTeleportMode::TeleportAndReset;
	}
	
	Proxy.SetAnimationAsset(NewAsset, GetSkelMeshComponent(), bInIsLooping, InPlayRate);

	// if composite, we want to make sure this is valid
	// this is due to protect recursive created composite
	// however, if we support modifying asset outside of viewport, it will have to be called whenever modified
	if (UAnimCompositeBase* CompositeBase = Cast<UAnimCompositeBase>(NewAsset))
	{
		CompositeBase->InvalidateRecursiveAsset();
	}

	UAnimMontage* Montage = Cast<UAnimMontage>(NewAsset);
	if ( Montage!=NULL )
	{
		Proxy.ReinitializeSlotNodes();
		if ( Montage->SlotAnimTracks.Num() > 0 )
		{
			for (const FSlotAnimationTrack& SlotAnimationTrack : Montage->SlotAnimTracks)
			{
				Proxy.RegisterSlotNodeWithAnimInstance(SlotAnimationTrack.SlotName);
			}
			Proxy.SetMontagePreviewSlot(Montage->SlotAnimTracks[0].SlotName);
		}
		RestartMontage( Montage );
		SetPlaying(IsPlaying());
	}
	else
	{
		// otherwise stop all montages
		StopAllMontages(0.25f);
	}
}

UAnimationAsset* UAnimSingleNodeInstance::GetAnimationAsset() const
{
	return CurrentAsset;
}

void UAnimSingleNodeInstance::SetPreviewCurveOverride(const FName& PoseName, float Value, bool bRemoveIfZero)
{
	GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().SetPreviewCurveOverride(PoseName, Value, bRemoveIfZero);
}

void UAnimSingleNodeInstance::SetMontageLoop(UAnimMontage* Montage, bool bIsLooping, FName StartingSection)
{
	check (Montage);

	int32 TotalSection = Montage->CompositeSections.Num();
	if( TotalSection > 0 )
	{
		if (StartingSection == NAME_None)
		{
			StartingSection = Montage->CompositeSections[0].SectionName;
		}
		FName FirstSection = StartingSection;
		FName LastSection = StartingSection;

		bool bSucceeded = false;
		// find last section
		int32 CurSection = Montage->GetSectionIndex(FirstSection);

		int32 Count = TotalSection;
		while( Count-- > 0 )
		{
			FName NewLastSection = Montage->CompositeSections[CurSection].NextSectionName;
			CurSection = Montage->GetSectionIndex(NewLastSection);

			if( CurSection != INDEX_NONE )
			{
				// used to rebuild next/prev
				Montage_SetNextSection(LastSection, NewLastSection);
				LastSection = NewLastSection;
			}
			else
			{
				bSucceeded = true;
				break;
			}
		}

		if( bSucceeded )
		{
			if ( bIsLooping )
			{
				Montage_SetNextSection(LastSection, FirstSection);
			}
			else
			{
				Montage_SetNextSection(LastSection, NAME_None);
			}
		}
		// else the default is already looping
	}
}

void UAnimSingleNodeInstance::SetMontagePreviewSlot(FName PreviewSlot)
{
	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
	Proxy.SetMontagePreviewSlot(PreviewSlot);
}

void UAnimSingleNodeInstance::UpdateMontageWeightForTimeSkip(float TimeDifference)
{
	if (!ensureMsgf(TimeDifference >= 0, TEXT("Cannot update montage weights with a negative time difference [Passed in TimeDifference: %f]"), TimeDifference))
	{
		TimeDifference = 0.0f;
	}
	
	Montage_UpdateWeight(TimeDifference);
	if (UAnimMontage * Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();

		UpdateMontageEvaluationData();

		if(Montage->SlotAnimTracks.Num() > 0)
		{
			const FName CurrentSlotNodeName = Montage->SlotAnimTracks[0].SlotName;
			Proxy.UpdateMontageWeightForSlot(CurrentSlotNodeName, 1.f);
		}
	}
}

void UAnimSingleNodeInstance::UpdateBlendspaceSamples(FVector InBlendInput)
{
	GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().UpdateBlendspaceSamples(InBlendInput);
}

void UAnimSingleNodeInstance::RestartMontage(UAnimMontage* Montage, FName FromSection)
{
	if(Montage == CurrentAsset)
	{
		FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();

		Proxy.ResetWeightInfo();
		Montage_Play(Montage, Proxy.GetPlayRate());
		if( FromSection != NAME_None )
		{
			Montage_JumpToSection(FromSection);
		}
		SetMontageLoop(Montage, Proxy.IsLooping(), FromSection);
		UpdateMontageWeightForTimeSkip(Montage->BlendIn.GetBlendTime());
	}
}

void UAnimSingleNodeInstance::NativeInitializeAnimation()
{
	USkeletalMeshComponent* SkelComp = GetSkelMeshComponent();
	SkelComp->AnimationData.Initialize(this);
}

void UAnimSingleNodeInstance::NativePostEvaluateAnimation()
{
	PostEvaluateAnimEvent.ExecuteIfBound();

	Super::NativePostEvaluateAnimation();
}

void UAnimSingleNodeInstance::OnMontageInstanceStopped(FAnimMontageInstance& StoppedMontageInstance) 
{
	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
	if (StoppedMontageInstance.Montage == CurrentAsset)
	{
		Proxy.SetCurrentTime(StoppedMontageInstance.GetPosition());
	}

	Super::OnMontageInstanceStopped(StoppedMontageInstance);
}

void UAnimSingleNodeInstance::Montage_Advance(float DeltaTime)
{
	Super::Montage_Advance(DeltaTime);
		
	FAnimMontageInstance * CurMontageInstance = GetActiveMontageInstance();
	if ( CurMontageInstance )
	{
		FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
		Proxy.SetCurrentTime(CurMontageInstance->GetPosition());
	}
}

void UAnimSingleNodeInstance::SetMirrorDataTable(const UMirrorDataTable* MirrorDataTable)
{
	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
	Proxy.SetMirrorDataTable(MirrorDataTable);
}

const UMirrorDataTable* UAnimSingleNodeInstance::GetMirrorDataTable()
{
	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
	return Proxy.GetMirrorDataTable();
}


void UAnimSingleNodeInstance::PlayAnim(bool bIsLooping, float InPlayRate, float InStartPosition)
{
	SetPlaying(true);
	SetLooping(bIsLooping);
	SetPlayRate(InPlayRate);
	SetPosition(InStartPosition);
}

void UAnimSingleNodeInstance::StopAnim()
{
	SetPlaying(false);
}

void UAnimSingleNodeInstance::SetLooping(bool bIsLooping)
{
	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
	Proxy.SetLooping(bIsLooping);

	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		SetMontageLoop(Montage, Proxy.IsLooping());
	}
}

void UAnimSingleNodeInstance::SetPlaying(bool bIsPlaying)
{
	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
	Proxy.SetPlaying(bIsPlaying);

	if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
	{
		CurMontageInstance->bPlaying = bIsPlaying;
	}
	else if (Proxy.IsPlaying())
	{
		UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset);
		if (Montage)
		{
			RestartMontage(Montage);
		}
	}
}

bool UAnimSingleNodeInstance::IsPlaying() const
{
	// since setPlaying is setting to montage, we should get it as symmmetry
	if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
	{
		return CurMontageInstance->bPlaying;
	}

	return GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().IsPlaying();
}

bool UAnimSingleNodeInstance::IsReverse() const
{
	return GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().IsReverse();
}

bool UAnimSingleNodeInstance::IsLooping() const
{
	return GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().IsLooping();
}

float UAnimSingleNodeInstance::GetPlayRate() const
{
	return GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().GetPlayRate();
}

void UAnimSingleNodeInstance::SetPlayRate(float InPlayRate)
{
	GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().SetPlayRate(InPlayRate);

	if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
	{
		CurMontageInstance->SetPlayRate(InPlayRate);
	}
}

UAnimationAsset* UAnimSingleNodeInstance::GetCurrentAsset()
{
	return CurrentAsset;
}

float UAnimSingleNodeInstance::GetCurrentTime() const
{
	return GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().GetCurrentTime();
}

void UAnimSingleNodeInstance::SetReverse(bool bInReverse)
{
	GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().SetReverse(bInReverse);

	if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
	{
		CurMontageInstance->SetPlayRate(GetPlayRate());
	}
}

void UAnimSingleNodeInstance::SetPositionWithPreviousTime(float InPosition, float InPreviousTime, bool bFireNotifies)
{
	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();

	Proxy.SetCurrentTime(FMath::Clamp<float>(InPosition, 0.f, GetLength()));

	if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
	{
		CurMontageInstance->SetPosition(Proxy.GetCurrentTime());
	}

	// Handle notifies
	// the way AnimInstance handles notifies doesn't work for single node because this does not tick or anything
	// this will need to handle manually, emptying, it and collect it, and trigger them at once. 
	if (bFireNotifies)
	{
		UAnimSequenceBase * SequenceBase = Cast<UAnimSequenceBase> (CurrentAsset);
		if (SequenceBase)
		{
			FAnimTickRecord TickRecord;
			TickRecord.TimeAccumulator = &InPosition;
			FAnimNotifyContext NotifyContext(TickRecord);
			NotifyQueue.Reset(GetSkelMeshComponent());

			SequenceBase->GetAnimNotifiesFromDeltaPositions(InPreviousTime, Proxy.GetCurrentTime(), NotifyContext);
			if ( NotifyContext.ActiveNotifies.Num() > 0 )
			{
				// single node instance only has 1 asset at a time
				NotifyQueue.AddAnimNotifies(NotifyContext.ActiveNotifies, 1.0f);
			}

			TriggerAnimNotifies(0.f);

		}
	}
}

void UAnimSingleNodeInstance::SetPosition(float InPosition, bool bFireNotifies)
{
	FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();

	float PreviousTime = Proxy.GetCurrentTime();

	SetPositionWithPreviousTime(InPosition, PreviousTime, bFireNotifies);
}

void UAnimSingleNodeInstance::SetBlendSpacePosition(const FVector& InPosition)
{
	GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().SetBlendSpacePosition(InPosition);
}

void UAnimSingleNodeInstance::SetInterpolationOverride(TOptional<EAnimInterpolationType> InterpolationType)
{
	GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().SetInterpolationOverride(InterpolationType);
}

TOptional<EAnimInterpolationType> UAnimSingleNodeInstance::GetInterpolationOverride() const
{
	return GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().GetInterpolationOverride();
}

void UAnimSingleNodeInstance::GetBlendSpaceState(FVector& OutPosition, FVector& OutFilteredPosition) const
{
	GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>().GetBlendSpaceState(OutPosition, OutFilteredPosition);
}

float UAnimSingleNodeInstance::GetLength()
{
	if ((CurrentAsset != NULL))
	{
		if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(CurrentAsset))
		{
			// Blend space length is normalized to 1 when getting and setting
			return 1.0f;
		}
		else if (UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(CurrentAsset))
		{
			return SequenceBase->GetPlayLength();
		}
	}
	return 0.f;
}

void UAnimSingleNodeInstance::StepForward()
{
	if (UAnimSequenceBase* Sequence = Cast<UAnimSequenceBase>(CurrentAsset))
	{
		FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();

		const FFrameRate& FrameRate = Sequence->GetSamplingFrameRate();
		const FFrameNumber LastSequenceFrameNumber = FrameRate.AsFrameTime(Sequence->GetPlayLength()).RoundToFrame();
		
		// Step forward a small amount and ceil to the next frame number 
		FFrameNumber StepToFrame = FrameRate.AsFrameTime(Proxy.GetCurrentTime() + UE_KINDA_SMALL_NUMBER).CeilToFrame();		
		if (IsLooping())
		{
			// Wrap around to start of the sequence
			StepToFrame %= (LastSequenceFrameNumber + 1);
		}

		StepToFrame.Value = FMath::Clamp<int32>(StepToFrame.Value, 0, LastSequenceFrameNumber.Value);

		SetPosition(FrameRate.AsSeconds(StepToFrame));
	}
	else if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(CurrentAsset))
	{
		// BlendSpace combines animations so there's no such thing as a frame. However, 1/30 is a sensible/common rate.
		FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
		float Length = Proxy.GetBlendSpaceLength();
		if (Length > 0.0f)
		{
			const float FixedFrameRate = 30.0f;
			float NormalizedDt = 1.0f / (FixedFrameRate * Length);
			float NormalizedTime = Proxy.GetCurrentTime() + NormalizedDt;
			NormalizedTime = IsLooping() ? 
				FMath::Wrap(NormalizedTime, 0.0f, 1.0f) : FMath::Clamp(NormalizedTime, 0.0f, 1.0f);
			SetPosition(NormalizedTime);
		}
	}
}

void UAnimSingleNodeInstance::StepBackward()
{
	if (UAnimSequenceBase* Sequence = Cast<UAnimSequenceBase>(CurrentAsset))
	{
		FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();

		const FFrameRate& FrameRate = Sequence->GetSamplingFrameRate();
		const FFrameNumber LastSequenceFrameNumber = FrameRate.AsFrameTime(Sequence->GetPlayLength()).RoundToFrame();

		// Step backwards a small amount and floor to the previous frame number 
		FFrameNumber StepToFrame = FrameRate.AsFrameTime(Proxy.GetCurrentTime() - UE_KINDA_SMALL_NUMBER).FloorToFrame();
		if (IsLooping())
		{
			// Wrap around to end of sequence
			StepToFrame = StepToFrame.Value < 0 ? LastSequenceFrameNumber : StepToFrame;
		}

		StepToFrame.Value = FMath::Clamp<int32>(StepToFrame.Value, 0, LastSequenceFrameNumber.Value);

		SetPosition(FrameRate.AsSeconds(StepToFrame));
	}
	else if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(CurrentAsset))
	{
		// BlendSpace combines animations so there's no such thing as a frame. However, 1/30 is a sensible/common rate.
		FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
		float Length = Proxy.GetBlendSpaceLength();
		if (Length > 0.0f)
		{
			const float FixedFrameRate = 30.0f;
			float NormalizedDt = 1.0f / (FixedFrameRate * Length);
			float NormalizedTime = Proxy.GetCurrentTime() - NormalizedDt;
			NormalizedTime = IsLooping() ? 
				FMath::Wrap(NormalizedTime, 0.0f, 1.0f) : FMath::Clamp(NormalizedTime, 0.0f, 1.0f);
			SetPosition(NormalizedTime);
		}
	}
}

FAnimInstanceProxy* UAnimSingleNodeInstance::CreateAnimInstanceProxy()
{
	return new FAnimSingleNodeInstanceProxy(this);
}

FVector UAnimSingleNodeInstance::GetFilterLastOutput()
{
	if (UBlendSpace* Blendspace = Cast<UBlendSpace>(CurrentAsset))
	{
		FAnimSingleNodeInstanceProxy& Proxy = GetProxyOnGameThread<FAnimSingleNodeInstanceProxy>();
		return Proxy.GetFilterLastOutput();
	}

	return FVector::ZeroVector;
}

