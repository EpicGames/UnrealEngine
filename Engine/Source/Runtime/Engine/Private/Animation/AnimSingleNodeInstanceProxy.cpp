// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSingleNodeInstanceProxy.h"
#include "Animation/AnimMontageEvaluationState.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimTrace.h"
#include "Animation/BlendSpace.h"
#include "Animation/PoseAsset.h"
#include "Animation/AnimStreamable.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimSyncScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSingleNodeInstanceProxy)

FAnimSingleNodeInstanceProxy::~FAnimSingleNodeInstanceProxy()
{
}

void FAnimSingleNodeInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::Initialize(InAnimInstance);

	CurrentAsset = nullptr;
#if WITH_EDITORONLY_DATA
	PreviewPoseCurrentTime = 0.0f;
#endif

	UpdateCounter.Reset();

	// it's already doing it when evaluate
	BlendSpacePosition = FVector::ZeroVector;

	CurrentTime = 0.f;
	DeltaTimeRecord = FDeltaTimeRecord();

	// initialize node manually 
	FAnimationInitializeContext InitContext(this);
	SingleNode.Initialize_AnyThread(InitContext);
}

bool FAnimSingleNodeInstanceProxy::Evaluate(FPoseContext& Output)
{
	SingleNode.Evaluate_AnyThread(Output);

	return true;
}

void FAnimSingleNodeInstanceProxy::SetMirrorDataTable(const UMirrorDataTable* InMirrorDataTable)
{
	MirrorDataTable = InMirrorDataTable;
}

const UMirrorDataTable* FAnimSingleNodeInstanceProxy::GetMirrorDataTable()
{
	return MirrorDataTable;
}

#if WITH_EDITORONLY_DATA

void FAnimSingleNodeInstanceProxy::PropagatePreviewCurve(FPoseContext& Output) 
{
	USkeleton* MySkeleton = GetSkeleton();

	FBlendedCurve Curve;
	UE::Anim::FCurveUtils::BuildUnsorted(Curve, PreviewCurveOverride);

	Output.Curve.Combine(Curve);
}
#endif // WITH_EDITORONLY_DATA

void FAnimSingleNodeInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	UpdateCounter.Increment();

	SingleNode.Update_AnyThread(InContext);
}

void FAnimSingleNodeInstanceProxy::PostUpdate(UAnimInstance* InAnimInstance) const
{
	FAnimInstanceProxy::PostUpdate(InAnimInstance);

	// sync up playing state for active montage instances
	int32 EvaluationDataIndex = 0;
	const TArray<FMontageEvaluationState>& EvaluationData = GetMontageEvaluationData();
	for (FAnimMontageInstance* MontageInstance : InAnimInstance->MontageInstances)
	{
		if (MontageInstance->Montage && MontageInstance->GetWeight() > ZERO_ANIMWEIGHT_THRESH)
		{
			// sanity check we are playing the same montage
			check(MontageInstance->Montage == EvaluationData[EvaluationDataIndex].Montage);
			MontageInstance->bPlaying = EvaluationData[EvaluationDataIndex].bIsPlaying;
			EvaluationDataIndex++;
		}
	}
}

void FAnimSingleNodeInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) 
{
	FAnimInstanceProxy::PreUpdate(InAnimInstance, DeltaSeconds);
#if WITH_EDITOR
	// @fixme only do this in pose asset
// 	// copy data to PreviewPoseOverride
// 	TMap<FName, float> PoseCurveList;
// 
// 	InAnimInstance->GetAnimationCurveList(ACF_DrivesPose, PoseCurveList);
// 
// 	if (PoseCurveList.Num() > 0)
// 	{
// 		PreviewPoseOverride.Append(PoseCurveList);
// 	}
#endif // WITH_EDITOR
}
void FAnimSingleNodeInstanceProxy::InitializeObjects(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::InitializeObjects(InAnimInstance);

	UAnimSingleNodeInstance* AnimSingleNodeInstance = CastChecked<UAnimSingleNodeInstance>(InAnimInstance);
	CurrentAsset = AnimSingleNodeInstance->CurrentAsset;

	InterpolationOverride = AnimSingleNodeInstance->GetInterpolationOverride();
}

void FAnimSingleNodeInstanceProxy::ClearObjects()
{
	FAnimInstanceProxy::ClearObjects();

	CurrentAsset = nullptr;
}

void FAnimSingleNodeInstanceProxy::SetPreviewCurveOverride(const FName& PoseName, float Value, bool bRemoveIfZero)
{
	float *CurveValPtr = PreviewCurveOverride.Find(PoseName);
	bool bShouldAddToList = bRemoveIfZero == false || FPlatformMath::Abs(Value) > ZERO_ANIMWEIGHT_THRESH;
	if (bShouldAddToList)
	{
		if (CurveValPtr)
		{
			// sum up, in the future we might normalize, but for now this just sums up
			// this won't work well if all of them have full weight - i.e. additive 
			*CurveValPtr = Value;
		}
		else
		{
			PreviewCurveOverride.Add(PoseName, Value);
		}
	}
	// if less than ZERO_ANIMWEIGHT_THRESH
	// no reason to keep them on the list
	else 
	{
		// remove if found
		PreviewCurveOverride.Remove(PoseName);
	}
}

void FAnimSingleNodeInstanceProxy::UpdateMontageWeightForSlot(const FName CurrentSlotNodeName, float InGlobalNodeWeight)
{
	GetSlotWeight(CurrentSlotNodeName, WeightInfo.SlotNodeWeight, WeightInfo.SourceWeight, WeightInfo.TotalNodeWeight);
	UpdateSlotNodeWeight(CurrentSlotNodeName, WeightInfo.SlotNodeWeight, InGlobalNodeWeight);
}

void FAnimSingleNodeInstanceProxy::SetMontagePreviewSlot(FName PreviewSlot)
{
	SingleNode.ActiveMontageSlot = PreviewSlot;
}

void FAnimSingleNodeInstanceProxy::InternalBlendSpaceEvaluatePose(class UBlendSpace* BlendSpace, TArray<FBlendSampleData>& BlendSampleDataCache, FPoseContext& OutContext)
{
	FAnimationPoseData AnimationPoseData = { OutContext.Pose, OutContext.Curve, OutContext.CustomAttributes };

	FAnimExtractContext ExtractionContext(static_cast<double>(CurrentTime), ShouldExtractRootMotion(), DeltaTimeRecord, bLooping);
	ExtractionContext.InterpolationOverride = InterpolationOverride;

	if (BlendSpace->IsValidAdditive())
	{
		FCompactPose& OutPose = OutContext.Pose;
		FBlendedCurve& OutCurve = OutContext.Curve;
		FCompactPose AdditivePose;
		FBlendedCurve AdditiveCurve;
		UE::Anim::FStackAttributeContainer AdditiveAttributes;
		AdditivePose.SetBoneContainer(&OutPose.GetBoneContainer());
		AdditiveCurve.InitFrom(OutCurve);
#if WITH_EDITORONLY_DATA
		if (BlendSpace->PreviewBasePose)
		{
			FAnimExtractContext Context(static_cast<double>(PreviewPoseCurrentTime));
			Context.InterpolationOverride = InterpolationOverride;
			BlendSpace->PreviewBasePose->GetBonePose(AnimationPoseData, Context);
		}
		else
#endif // WITH_EDITORONLY_DATA
		{
			// otherwise, get ref pose
			OutPose.ResetToRefPose();
		}

		FAnimationPoseData AdditiveAnimationPoseData = { AdditivePose, AdditiveCurve, AdditiveAttributes };
		BlendSpace->GetAnimationPose(BlendSampleDataCache, ExtractionContext, AdditiveAnimationPoseData);

		enum EAdditiveAnimationType AdditiveType = 
			BlendSpace->bContainsRotationOffsetMeshSpaceSamples ? AAT_RotationOffsetMeshSpace : AAT_LocalSpaceBase;
		FAnimationRuntime::AccumulateAdditivePose(AnimationPoseData, AdditiveAnimationPoseData, 1.f, AdditiveType);
	}
	else
	{
		BlendSpace->GetAnimationPose(BlendSampleDataCache, ExtractionContext, AnimationPoseData);
	}
}

void FAnimSingleNodeInstanceProxy::SetAnimationAsset(class UAnimationAsset* NewAsset, USkeletalMeshComponent* MeshComponent, bool bIsLooping, float InPlayRate)
{
	bLooping = bIsLooping;
	PlayRate = InPlayRate;
	CurrentTime = 0.f;
	BlendSpacePosition = FVector::ZeroVector;
	BlendSampleData.Reset();
	MarkerTickRecord.Reset();
	ResetSync();
	UpdateBlendspaceSamples(BlendSpacePosition);

#if WITH_EDITORONLY_DATA
	PreviewPoseCurrentTime = 0.0f;
	PreviewCurveOverride.Reset();
#endif

	
	if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(NewAsset))
	{
		BlendSpace->InitializeFilter(&BlendFilter);
	}
}

void FAnimSingleNodeInstanceProxy::UpdateBlendspaceSamples(FVector InBlendInput)
{
	if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(CurrentAsset))
	{
		float OutCurrentTime = 0.f;
		FMarkerTickRecord TempMarkerTickRecord;
		BlendSpaceAdvanceImmediate(BlendSpace, InBlendInput, BlendSampleData, BlendFilter, false, 1.f, 0.f, OutCurrentTime, TempMarkerTickRecord);
	}
}

void FAnimSingleNodeInstanceProxy::SetReverse(bool bInReverse)
{
	bReverse = bInReverse;
	if (bInReverse)
	{
		PlayRate = -FMath::Abs(PlayRate);
	}
	else
	{
		PlayRate = FMath::Abs(PlayRate);
	}

// reverse support is a bit tricky for montage
// since we don't have delegate when it reached to the beginning
// for now I comment this out and do not support
// I'd like the buttons to be customizable per asset types -
// 	TTP 233456	ANIM: support different scrub controls per asset type
/*
	FAnimMontageInstance * CurMontageInstance = GetActiveMontageInstance();
	if ( CurMontageInstance )
	{
		if ( bReverse == (CurMontageInstance->PlayRate > 0.f) )
		{
			CurMontageInstance->PlayRate *= -1.f;
		}
	}*/
}

void FAnimSingleNodeInstanceProxy::SetBlendSpacePosition(const FVector& InPosition)
{
	BlendSpacePosition = InPosition;
}

void FAnimSingleNodeInstanceProxy::GetBlendSpaceState(FVector& OutPosition, FVector& OutFilteredPosition) const
{
	OutFilteredPosition = BlendFilter.GetFilterLastOutput();
	OutPosition = BlendSpacePosition;
}

float FAnimSingleNodeInstanceProxy::GetBlendSpaceLength() const
{
	float TotalLength = 0.0f;
	float TotalWeight = 0.0f;
	for (int32 Index = 0 ; Index != BlendSampleData.Num() ; ++Index)
	{
		const FBlendSampleData& Data = BlendSampleData[Index];
		float AnimLength = Data.Animation->GetPlayLength();
		float Weight = Data.GetClampedWeight();
		TotalLength += AnimLength * Weight;
		TotalWeight += Weight;
	}
	if (TotalWeight > 0.0f)
	{
		return TotalLength / TotalWeight;
	}
	return 0.0f;
}

void FAnimNode_SingleNode::Evaluate_AnyThread(FPoseContext& Output)
{
	const bool bCanProcessAdditiveAnimationsLocal
#if WITH_EDITOR
		= Proxy->bCanProcessAdditiveAnimations;
#else
		= false;
#endif

	if (Proxy->CurrentAsset != nullptr && !Proxy->CurrentAsset->HasAnyFlags(RF_BeginDestroyed))
	{
		FAnimationPoseData OutputAnimationPoseData(Output);

		//@TODO: animrefactor: Seems like more code duplication than we need
		if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(Proxy->CurrentAsset))
		{
			Proxy->InternalBlendSpaceEvaluatePose(BlendSpace, Proxy->BlendSampleData, Output);
		}
		else if (UAnimSequence* Sequence = Cast<UAnimSequence>(Proxy->CurrentAsset))
		{
			FAnimExtractContext ExtractionContext(static_cast<double>(Proxy->CurrentTime), Sequence->bEnableRootMotion, Proxy->DeltaTimeRecord, Proxy->bLooping);
#if WITH_EDITOR
			ExtractionContext.bIgnoreRootLock = Proxy->bIgnoreRootLock;
			ExtractionContext.bExtractRootMotion = Proxy->bIgnoreRootMotion ? false : ExtractionContext.bExtractRootMotion;
#endif
			ExtractionContext.InterpolationOverride = Proxy->InterpolationOverride;

			if (Sequence->IsValidAdditive())
			{
				if (bCanProcessAdditiveAnimationsLocal)
				{
					Sequence->GetAdditiveBasePose(OutputAnimationPoseData, ExtractionContext);
				}
				else
				{
					Output.ResetToRefPose();
				}

				FCompactPose AdditivePose;
				FBlendedCurve AdditiveCurve;
				UE::Anim::FStackAttributeContainer AdditiveAttributes;
				AdditivePose.SetBoneContainer(&Output.Pose.GetBoneContainer());
				AdditiveCurve.InitFrom(Output.Curve);

				FAnimationPoseData AdditivePoseData = { AdditivePose, AdditiveCurve, AdditiveAttributes };
				Sequence->GetAnimationPose(AdditivePoseData, ExtractionContext);
				FAnimationRuntime::AccumulateAdditivePose(OutputAnimationPoseData, AdditivePoseData, 1.f, Sequence->AdditiveAnimType);

				Output.Pose.NormalizeRotations();
			}
			else
			{
				// if SkeletalMesh isn't there, we'll need to use skeleton
				Sequence->GetAnimationPose(OutputAnimationPoseData, ExtractionContext);
			}
		}
		else if (UAnimStreamable* Streamable = Cast<UAnimStreamable>(Proxy->CurrentAsset))
		{
			// No Additive support yet
			/*if (Streamable->IsValidAdditive())
			{
				FAnimExtractContext ExtractionContext(Proxy->CurrentTime, Streamable->bEnableRootMotion);

				if (bCanProcessAdditiveAnimationsLocal)
				{
					Sequence->GetAdditiveBasePose(Output.Pose, Output.Curve, ExtractionContext);
				}
				else
				{
					Output.ResetToRefPose();
				}

				FCompactPose AdditivePose;
				FBlendedCurve AdditiveCurve;
				AdditivePose.SetBoneContainer(&Output.Pose.GetBoneContainer());
				AdditiveCurve.InitFrom(Output.Curve);
				Streamable->GetAnimationPose(AdditivePose, AdditiveCurve, ExtractionContext);

				FAnimationRuntime::AccumulateAdditivePose(Output.Pose, AdditivePose, Output.Curve, AdditiveCurve, 1.f, Sequence->AdditiveAnimType);
				Output.Pose.NormalizeRotations();
			}
			else*/
			{
				// if SkeletalMesh isn't there, we'll need to use skeleton
				FAnimExtractContext Context(static_cast<double>(Proxy->CurrentTime), Streamable->bEnableRootMotion, Proxy->DeltaTimeRecord, Proxy->bLooping);
				Context.InterpolationOverride = Proxy->InterpolationOverride;
				Streamable->GetAnimationPose(OutputAnimationPoseData, Context);
			}
		}
		else if (UAnimComposite* Composite = Cast<UAnimComposite>(Proxy->CurrentAsset))
		{
			FAnimExtractContext ExtractionContext(static_cast<double>(Proxy->CurrentTime), Proxy->ShouldExtractRootMotion(), Proxy->DeltaTimeRecord, Proxy->bLooping);
			ExtractionContext.InterpolationOverride = Proxy->InterpolationOverride;
			const FAnimTrack& AnimTrack = Composite->AnimationTrack;

			// find out if this is additive animation
			if (AnimTrack.IsAdditive())
			{
#if WITH_EDITORONLY_DATA
				if (bCanProcessAdditiveAnimationsLocal && Composite->PreviewBasePose)
				{
					Composite->PreviewBasePose->GetAdditiveBasePose(OutputAnimationPoseData, ExtractionContext);
				}
				else
#endif
				{
					// get base pose - for now we only support ref pose as base
					Output.Pose.ResetToRefPose();
				}

				EAdditiveAnimationType AdditiveAnimType = AnimTrack.IsRotationOffsetAdditive()? AAT_RotationOffsetMeshSpace : AAT_LocalSpaceBase;

				FCompactPose AdditivePose;
				FBlendedCurve AdditiveCurve;
				UE::Anim::FStackAttributeContainer AdditiveAttributes;
				AdditivePose.SetBoneContainer(&Output.Pose.GetBoneContainer());
				AdditiveCurve.InitFrom(Output.Curve);

				FAnimationPoseData AdditiveAnimationPoseData = { AdditivePose, AdditiveCurve, AdditiveAttributes };
				Composite->GetAnimationPose(AdditiveAnimationPoseData, ExtractionContext);

				FAnimationRuntime::AccumulateAdditivePose(OutputAnimationPoseData, AdditiveAnimationPoseData, 1.f, AdditiveAnimType);
			}
			else
			{
				//doesn't handle additive yet
				Composite->GetAnimationPose(OutputAnimationPoseData, ExtractionContext);
			}
		}
		else if (UAnimMontage* Montage = Cast<UAnimMontage>(Proxy->CurrentAsset))
		{
			// for now only update first slot
			// in the future, add option to see which slot to see
			if (Montage->SlotAnimTracks.Num() > 0)
			{
				FCompactPose LocalSourcePose;
				FBlendedCurve LocalSourceCurve;
				LocalSourcePose.SetBoneContainer(&Output.Pose.GetBoneContainer());
				LocalSourceCurve.InitFrom(Output.Curve);

				UE::Anim::FStackAttributeContainer LocalSourceAttributes;
			
				FAnimTrack const* const AnimTrack = Montage->GetAnimationData(ActiveMontageSlot);
				if (AnimTrack && AnimTrack->IsAdditive())
				{
#if WITH_EDITORONLY_DATA
					// if montage is additive, we need to have base pose for the slot pose evaluate
					if (bCanProcessAdditiveAnimationsLocal && Montage->PreviewBasePose && Montage->GetPlayLength() > 0.f)
					{
						FAnimationPoseData LocalAnimationPoseData = { LocalSourcePose, LocalSourceCurve, LocalSourceAttributes };
						FAnimExtractContext Context(static_cast<double>(Proxy->CurrentTime));
						Context.InterpolationOverride = Proxy->InterpolationOverride;
						Montage->PreviewBasePose->GetBonePose(LocalAnimationPoseData, Context);
					}
					else
#endif // WITH_EDITORONLY_DATA
					{
						LocalSourcePose.ResetToRefPose();
					}
				}
				else
				{
					LocalSourcePose.ResetToRefPose();
				}

				const FAnimationPoseData LocalAnimationPoseData(LocalSourcePose, LocalSourceCurve, LocalSourceAttributes);
				Proxy->SlotEvaluatePose(ActiveMontageSlot, LocalAnimationPoseData, Proxy->WeightInfo.SourceWeight, OutputAnimationPoseData, Proxy->WeightInfo.SlotNodeWeight, Proxy->WeightInfo.TotalNodeWeight);
			}
		}
		else
		{
			// pose asset is handled by preview instance : pose blend node
			// and you can't drag pose asset to level to create single node instance. 
			Output.ResetToRefPose();
		}

#if WITH_EDITORONLY_DATA
		// have to propagate output curve before pose asset as it can use pose curve data
		Proxy->PropagatePreviewCurve(Output);

		// if it has a preview pose asset, we have to handle that after we do all animation
		if (const UPoseAsset* PoseAsset = Proxy->CurrentAsset->PreviewPoseAsset)
		{
			if (PoseAsset->GetSkeleton() != nullptr)
			{
				const TArray<FName>& PoseNames = PoseAsset->GetPoseFNames();

				int32 TotalPoses = PoseNames.Num();
				FAnimExtractContext ExtractContext;
				ExtractContext.PoseCurves.AddZeroed(TotalPoses);

				for (int32 PoseIndex = 0; PoseIndex <PoseNames.Num(); ++PoseIndex)
				{
					const FName& PoseName = PoseNames[PoseIndex];
					ExtractContext.PoseCurves[PoseIndex].PoseIndex = PoseIndex;
					ExtractContext.PoseCurves[PoseIndex].Value = Output.Curve.Get(PoseName);
				}

				if (PoseAsset->IsValidAdditive())
				{
					FCompactPose AdditivePose;
					FBlendedCurve AdditiveCurve;
					UE::Anim::FStackAttributeContainer AdditiveAttributes;
					AdditivePose.SetBoneContainer(&Output.Pose.GetBoneContainer());
					AdditiveCurve.InitFrom(Output.Curve);

					FAnimationPoseData AdditiveAnimationPoseData { AdditivePose, AdditiveCurve, AdditiveAttributes };
					PoseAsset->GetAnimationPose(AdditiveAnimationPoseData, ExtractContext);
					FAnimationRuntime::AccumulateAdditivePose(OutputAnimationPoseData, AdditiveAnimationPoseData, 1.f, EAdditiveAnimationType::AAT_LocalSpaceBase);
				}
				else
				{
					FPoseContext LocalCurrentPose(Output);
					FPoseContext LocalSourcePose(Output);

					LocalSourcePose = Output;

					FAnimationPoseData CurrentAnimationPoseData(LocalCurrentPose);
					if (PoseAsset->GetAnimationPose(CurrentAnimationPoseData, ExtractContext))
					{
						TArray<float> BoneBlendWeights;

						const TArray<FName>& TrackNames = PoseAsset->GetTrackNames();
						const FBoneContainer& BoneContainer = Output.Pose.GetBoneContainer();
						const TArray<FBoneIndexType>& RequiredBoneIndices = BoneContainer.GetBoneIndicesArray();
						BoneBlendWeights.AddZeroed(RequiredBoneIndices.Num());

						for (const auto& TrackName : TrackNames)
						{
							int32 MeshBoneIndex = BoneContainer.GetPoseBoneIndexForBoneName(TrackName);
							FCompactPoseBoneIndex CompactBoneIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
							if (CompactBoneIndex != INDEX_NONE)
							{
								BoneBlendWeights[CompactBoneIndex.GetInt()] = 1.f;
							}
						}

						// once we get it, we have to blend by weight
						FAnimationPoseData AnimationPoseData = { Output.Pose, Output.Curve, Output.CustomAttributes };

						const FAnimationPoseData SourceAnimationPoseData(LocalSourcePose);
						FAnimationRuntime::BlendTwoPosesTogetherPerBone(SourceAnimationPoseData, CurrentAnimationPoseData, BoneBlendWeights, AnimationPoseData);
					}
				}
			}
		}

		if (Proxy->MirrorDataTable)
		{
			FAnimationRuntime::MirrorPose(Output.Pose, *(Proxy->MirrorDataTable));
		}
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
		Output.ResetToRefPose();
#if WITH_EDITORONLY_DATA
		// even if you don't have any asset curve, we want to output this curve values
		Proxy->PropagatePreviewCurve(Output);
#endif // WITH_EDITORONLY_DATA
	}
}

void FAnimNode_SingleNode::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	float NewPlayRate = Proxy->PlayRate;
	UAnimSequence* PreviewBasePose = nullptr;

	if (Proxy->bPlaying == false)
	{
		// we still have to tick animation when bPlaying is false because 
		NewPlayRate = 0.f;
	}

	if(Proxy->CurrentAsset != nullptr)
	{
		UE::Anim::FAnimSyncGroupScope& SyncScope = Context.GetMessageChecked<UE::Anim::FAnimSyncGroupScope>();

		if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(Proxy->CurrentAsset))
		{
			FAnimTickRecord TickRecord(
				BlendSpace, Proxy->BlendSpacePosition, Proxy->BlendSampleData, Proxy->BlendFilter, Proxy->bLooping, 
				NewPlayRate, false, false, 1.f, /*inout*/ Proxy->CurrentTime, Proxy->MarkerTickRecord);
			TickRecord.DeltaTimeRecord = &(Proxy->DeltaTimeRecord);
#if ANIM_TRACE_ENABLED
			TickRecord.SourceProxy = Context.AnimInstanceProxy;
#endif
			SyncScope.AddTickRecord(TickRecord);

#if WITH_EDITORONLY_DATA
			PreviewBasePose = BlendSpace->PreviewBasePose;
#endif
		}
		else if (UAnimSequence* Sequence = Cast<UAnimSequence>(Proxy->CurrentAsset))
		{
			FAnimTickRecord TickRecord(Sequence, Proxy->bLooping, NewPlayRate, false, 1.f, /*inout*/ Proxy->CurrentTime, Proxy->MarkerTickRecord);
			TickRecord.DeltaTimeRecord = &(Proxy->DeltaTimeRecord);
#if ANIM_TRACE_ENABLED
			TickRecord.SourceProxy = Context.AnimInstanceProxy;
#endif
			SyncScope.AddTickRecord(TickRecord);

			// if it's not looping, just set play to be false when reached to end
			if (!Proxy->bLooping)
			{
				const float CombinedPlayRate = NewPlayRate*Sequence->RateScale;
				if ((CombinedPlayRate < 0.f && Proxy->CurrentTime <= 0.f) || (CombinedPlayRate > 0.f && Proxy->CurrentTime >= Sequence->GetPlayLength()))
				{
					Proxy->SetPlaying(false);
				}
			}
		}
		else if (UAnimStreamable* Streamable = Cast<UAnimStreamable>(Proxy->CurrentAsset))
		{
			FAnimTickRecord TickRecord(Streamable, Proxy->bLooping, NewPlayRate, false, 1.f, /*inout*/ Proxy->CurrentTime, Proxy->MarkerTickRecord);
			TickRecord.DeltaTimeRecord = &(Proxy->DeltaTimeRecord);
#if ANIM_TRACE_ENABLED
			TickRecord.SourceProxy = Context.AnimInstanceProxy;
#endif
			
			SyncScope.AddTickRecord(TickRecord);

			// if it's not looping, just set play to be false when reached to end
			if (!Proxy->bLooping)
			{
				const float CombinedPlayRate = NewPlayRate * Streamable->RateScale;
				if ((CombinedPlayRate < 0.f && Proxy->CurrentTime <= 0.f) || (CombinedPlayRate > 0.f && Proxy->CurrentTime >= Streamable->GetPlayLength()))
				{
					Proxy->SetPlaying(false);
				}
			}
		}
		else if(UAnimComposite* Composite = Cast<UAnimComposite>(Proxy->CurrentAsset))
		{
			FAnimTickRecord TickRecord(Composite, Proxy->bLooping, NewPlayRate, false, 1.f, /*inout*/ Proxy->CurrentTime, Proxy->MarkerTickRecord);
			TickRecord.DeltaTimeRecord = &(Proxy->DeltaTimeRecord);
#if ANIM_TRACE_ENABLED
			TickRecord.SourceProxy = Context.AnimInstanceProxy;
#endif
			
			SyncScope.AddTickRecord(TickRecord);

			// if it's not looping, just set play to be false when reached to end
			if (!Proxy->bLooping)
			{
				const float CombinedPlayRate = NewPlayRate*Composite->RateScale;
				if ((CombinedPlayRate < 0.f && Proxy->CurrentTime <= 0.f) || (CombinedPlayRate > 0.f && Proxy->CurrentTime >= Composite->GetPlayLength()))
				{
					Proxy->SetPlaying(false);
				}
			}
		}
		else if (UAnimMontage * Montage = Cast<UAnimMontage>(Proxy->CurrentAsset))
		{
			// Full weight , if you don't have slot track, you won't be able to see animation playing
			if ( Montage->SlotAnimTracks.Num() > 0 )
			{
				Proxy->UpdateMontageWeightForSlot(ActiveMontageSlot, 1.f);
			}
			// get the montage position
			// @todo anim: temporarily just choose first slot and show the location
			const FMontageEvaluationState* ActiveMontageEvaluationState = Proxy->GetActiveMontageEvaluationState();
			if (ActiveMontageEvaluationState)
			{
				Proxy->CurrentTime = ActiveMontageEvaluationState->MontagePosition;
				Proxy->DeltaTimeRecord = ActiveMontageEvaluationState->DeltaTimeRecord;
			}
			else if (Proxy->bPlaying)
			{
				Proxy->SetPlaying(false);
			}
#if WITH_EDITORONLY_DATA
			PreviewBasePose = Montage->PreviewBasePose;
#endif
		}
		else if (UPoseAsset* PoseAsset = Cast<UPoseAsset>(Proxy->CurrentAsset))
		{
			FAnimTickRecord TickRecord(PoseAsset, 1.f);
			TickRecord.DeltaTimeRecord = &(Proxy->DeltaTimeRecord);
#if ANIM_TRACE_ENABLED
			TickRecord.SourceProxy = Context.AnimInstanceProxy;
#endif

			SyncScope.AddTickRecord(TickRecord);
		}
	}

#if WITH_EDITORONLY_DATA
	if(PreviewBasePose)
	{
		float MoveDelta = Proxy->GetDeltaSeconds() * NewPlayRate;
		const bool bIsPreviewPoseLooping = true;

		FAnimationRuntime::AdvanceTime(bIsPreviewPoseLooping, MoveDelta, Proxy->PreviewPoseCurrentTime, PreviewBasePose->GetPlayLength());
	}
#endif
}

