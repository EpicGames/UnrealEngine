// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimStateMachineTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStateMachineTypes)

/////////////////////////////////////////////////////
// UAnimStateMachineTypes

UAnimStateMachineTypes::UAnimStateMachineTypes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 FBakedAnimationStateMachine::FindStateIndex(const FName& InStateName) const
{
	for (int32 Index = 0; Index < States.Num(); Index++)
	{
		if (States[Index].StateName == InStateName)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

int32 FBakedAnimationStateMachine::FindTransitionIndex(const FName& InStateNameFrom, const FName& InStateNameTo) const
{
	return FindTransitionIndex(FindStateIndex(InStateNameFrom), FindStateIndex(InStateNameTo));
}

int32 FBakedAnimationStateMachine::FindTransitionIndex(const int32 InStateIdxFrom, const int32 InStateIdxTo) const
{
	// Early out if any request is invalid
	if(InStateIdxFrom == INDEX_NONE || InStateIdxTo == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 Count = Transitions.Num();
	for(int32 Index = 0 ; Index < Count ; ++Index)
	{
		const FAnimationTransitionBetweenStates& Transition = Transitions[Index];
		if(Transition.PreviousState == InStateIdxFrom && Transition.NextState == InStateIdxTo)
		{
			// If we match the request, output the index.
			return Index;
		}
	}

	return INDEX_NONE;
}

