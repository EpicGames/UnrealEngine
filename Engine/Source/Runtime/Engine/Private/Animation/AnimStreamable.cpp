// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimStreamable.cpp: Animation that can be streamed instead of being loaded completely
=============================================================================*/ 

#include "Animation/AnimStreamable.h"

#include "Animation/AnimCompress.h"
#include "Interfaces/ITargetPlatform.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Animation/AnimSequence.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "AnimationUtils.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectSaveContext.h"
#include "DerivedDataCacheInterface.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimCurveCompressionSettings.h"
#include "Animation/AnimCurveCompressionCodec.h"
#include "Animation/VariableFrameStrippingSettings.h"
#include "BonePose.h"
#include "CommonFrameRates.h"
#include "ContentStreaming.h"
#include "ITimeManagementModule.h"
#include "ProfilingDebugging/CookStats.h"
#include "Animation/AnimationPoseData.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include "Animation/AnimSequenceHelpers.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimData/IAnimationDataModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStreamable)

#if WITH_EDITOR
#include "Animation/AnimCompressionDerivedData.h"
#endif

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

DECLARE_CYCLE_STAT(TEXT("AnimStreamable GetAnimationPose"), STAT_AnimStreamable_GetAnimationPose, STATGROUP_Anim);

#if ENABLE_COOK_STATS
namespace AnimStreamableCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("AnimStreamable.Usage"), TEXT(""));
	});
}
#endif

// This is a version string for the streaming anim chunk logic
// If you want to bump this version, generate a new guid using VS->Tools->Create GUID and
// set it here
const TCHAR* StreamingAnimChunkVersion = TEXT("1F1656B9E10142729AB16650D9821B1F");

const float MINIMUM_CHUNK_SIZE = 4.0f;

float GChunkSizeSeconds = MINIMUM_CHUNK_SIZE;

static const TCHAR* ChunkSizeSecondsCVarName = TEXT("a.Streaming.ChunkSizeSeconds");

static FAutoConsoleVariableRef CVarChunkSizeSeconds(
	ChunkSizeSecondsCVarName,
	GChunkSizeSeconds,
	TEXT("Size of streaming animation chunk in seconds, 0 or negative signifies only have 1 chunk"));

void FAnimStreamableChunk::Serialize(FArchive& Ar, UAnimStreamable* Owner, int32 ChunkIndex)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FAnimStreamableChunk::Serialize"), STAT_StreamedAnimChunk_Serialize, STATGROUP_LoadTime);

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	Ar << StartTime;
	Ar << SequenceLength;

	if(bCooked)
	{
		if (ChunkIndex == 0)
		{
			// Chunk 0 just serializes the compressed data directly
			if (Ar.IsLoading())
			{
				check(!CompressedAnimSequence);
				CompressedAnimSequence = new FCompressedAnimSequence();
			}
			CompressedAnimSequence->SerializeCompressedData(Ar, false, Owner, Owner->GetSkeleton(), Owner->BoneCompressionSettings, Owner->CurveCompressionSettings, false);
		}
		else
		{
			BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);

			if (Ar.IsSaving())
			{
				//Need to pack compressed data into BulkData
				TArray<uint8> TempBytes;
				const int32 InitialSize = CompressedAnimSequence->CompressedDataStructure->GetApproxCompressedSize();
				TempBytes.Reset(InitialSize);

				FMemoryWriter TempAr(TempBytes, true);
				TempAr.SetFilterEditorOnly(Ar.IsFilterEditorOnly());
				CompressedAnimSequence->SerializeCompressedData(TempAr, false, Owner, Owner->GetSkeleton(), Owner->BoneCompressionSettings, Owner->CurveCompressionSettings, false);

				BulkData.Lock(LOCK_READ_WRITE);
				void* ChunkData = BulkData.Realloc(TempBytes.Num());
				FMemory::Memcpy(ChunkData, TempBytes.GetData(), TempBytes.Num());
				BulkData.Unlock();
			}

			// streaming doesn't use memory mapped IO
			BulkData.Serialize(Ar, Owner, ChunkIndex, false);
		}
	}
}

void FStreamableAnimPlatformData::Serialize(FArchive& Ar, UAnimStreamable* Owner)
{
	int32 NumChunks = Chunks.Num();
	Ar << NumChunks;

	if (Ar.IsLoading())
	{
		Chunks.Reset(NumChunks);
		Chunks.AddDefaulted(NumChunks);
	}
	for (int32 ChunkIndex = 0; ChunkIndex < NumChunks; ++ChunkIndex)
	{
		Chunks[ChunkIndex].Serialize(Ar, Owner, ChunkIndex);
	}
}

UAnimStreamable::UAnimStreamable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, RunningAnimPlatformData(nullptr)
#endif
{
	bUseRawDataOnly = true;
}

void UAnimStreamable::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();
	if (TargetPlatform)
	{
		RequestCompressedData(TargetPlatform); //Make sure target platform data is built
	}

#endif

	Super::PreSave(ObjectSaveContext);
}

void UAnimStreamable::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogAnimation, Fatal, "This platform requires cooked packages, and animation data was not cooked into %ls.", *GetFullName());
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::ReintroduceAnimationDataModelInterface)
	{
		// Figure out correct SamplingFrameRate value 
		if(SourceSequence && !SourceSequence->HasAnyFlags(RF_NeedPostLoad))
		{
			SamplingFrameRate = SourceSequence->GetSamplingFrameRate();
		}
		else
		{
			const int32 NumberOfFrames = FMath::Max(NumberOfKeys - 1, 1);

			// Generate the frame-rate according to the number of frames and sequence length
			const double DecimalFrameRate = (double)NumberOfFrames / ((double)GetPlayLength() > 0.0 ? (double)GetPlayLength() : 1.0);

			// Account for non-whole number frame rates using large denominator
			const double Denominator = 1000000.0;
			SamplingFrameRate = FFrameRate(DecimalFrameRate * Denominator, Denominator);	 

			// Try to simplifiy the frame rate, in case it is a multiple of the commonly used frame rates e.g. 10000/300000 -> 1/30
			TArrayView<const FCommonFrameRateInfo> CommonFrameRates = FModuleManager::LoadModulePtr<ITimeManagementModule>("TimeManagement")->GetAllCommonFrameRates();
			for (const FCommonFrameRateInfo& Info : CommonFrameRates)
			{
				const bool bDoesNotAlreadyMatch = Info.FrameRate.Denominator != SamplingFrameRate.Denominator && Info.FrameRate.Numerator != SamplingFrameRate.Numerator;
		
				if (bDoesNotAlreadyMatch && FMath::IsNearlyEqual(SamplingFrameRate.AsInterval(), Info.FrameRate.AsInterval()))
				{
					SamplingFrameRate = Info.FrameRate;
					break;
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	if (bCooked)
	{
		if (Ar.IsLoading())
		{
			GetRunningPlatformData().Serialize(Ar, this);
		}
		else
		{
			FStreamableAnimPlatformData& PlatformData = GetStreamingAnimPlatformData(Ar.CookingTarget());
			PlatformData.Serialize(Ar, this);
		}
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RenamingAnimationNumFrames)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NumberOfKeys = NumFrames;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITORONLY_DATA
}

void UAnimStreamable::HandleAssetPlayerTickedInternal(FAnimAssetTickContext &Context, const float PreviousTime, const float MoveDelta, const FAnimTickRecord &Instance, struct FAnimNotifyQueue& NotifyQueue) const
{
	Super::HandleAssetPlayerTickedInternal(Context, PreviousTime, MoveDelta, Instance, NotifyQueue);
	
	const int32 ChunkIndex = GetChunkIndexForTime(GetRunningPlatformData().Chunks, PreviousTime);
	IAnimationStreamingManager& StreamingManager = IStreamingManager::Get().GetAnimationStreamingManager();
	const FCompressedAnimSequence* CurveCompressedDataChunk = StreamingManager.GetLoadedChunk(this, ChunkIndex, true);
	
	//ExtractRootMotionFromTrack(AnimationTrack, PreviousTime, PreviousTime + MoveDelta, Context.RootMotionMovementParams);
}

FORCEINLINE int32 PreviousChunkIndex(int32 ChunkIndex, int32 NumChunks)
{
	return (ChunkIndex + NumChunks - 1) % NumChunks;
}

void UAnimStreamable::GetAnimationPose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const
{
	SCOPE_CYCLE_COUNTER(STAT_AnimStreamable_GetAnimationPose);
	CSV_SCOPED_TIMING_STAT(Animation, AnimStreamable_GetAnimationPose);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();

	OutPose.ResetToRefPose();

	const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();
	//const bool bUseRawDataForPoseExtraction = bForceUseRawData || UseRawDataForPoseExtraction(RequiredBones);

	const bool bIsBakedAdditive = false;//!bUseRawDataForPoseExtraction && IsValidAdditive();
	const EAdditiveAnimationType AdditiveType = AAT_None;

	const USkeleton* MySkeleton = GetSkeleton();
	if (!MySkeleton)
	{
		if (bIsBakedAdditive)
		{
			OutPose.ResetToAdditiveIdentity();
		}
		else
		{
			OutPose.ResetToRefPose();
		}
		return;
	}

	const bool bDisableRetargeting = RequiredBones.GetDisableRetargeting();

	// initialize with ref-pose
	if (bIsBakedAdditive)
	{
		//When using baked additive ref pose is identity
		OutPose.ResetToAdditiveIdentity();
	}
	else
	{
		// if retargeting is disabled, we initialize pose with 'Retargeting Source' ref pose.
		if (bDisableRetargeting)
		{
			TArray<FTransform> const& AuthoredOnRefSkeleton = MySkeleton->GetRefLocalPoses(RetargetSource);
			TArray<FBoneIndexType> const& RequireBonesIndexArray = RequiredBones.GetBoneIndicesArray();

			int32 const NumRequiredBones = RequireBonesIndexArray.Num();
			for (FCompactPoseBoneIndex PoseBoneIndex : OutPose.ForEachBoneIndex())
			{
				int32 const& SkeletonBoneIndex = RequiredBones.GetSkeletonIndex(PoseBoneIndex);

				// Pose bone index should always exist in Skeleton
				checkSlow(SkeletonBoneIndex != INDEX_NONE);
				OutPose[PoseBoneIndex] = AuthoredOnRefSkeleton[SkeletonBoneIndex];
			}
		}
		else
		{
			OutPose.ResetToRefPose();
		}
	}

	//FRootMotionReset RootMotionReset(bEnableRootMotion, RootMotionRootLock, bForceRootLock, ExtractRootTrackTransform(0.f, &RequiredBones), IsValidAdditive());
	FRootMotionReset RootMotionReset(bEnableRootMotion, RootMotionRootLock,
#if WITH_EDITOR
	!ExtractionContext.bIgnoreRootLock &&
#endif // WITH_EDITOR
	bForceRootLock,
	FTransform(), false); // MDW Does not support root motion yet

#if WITH_EDITOR
	if (IsDataModelValid() && (!HasRunningPlatformData() || RequiredBones.ShouldUseRawData()))
	{
		//Need to evaluate raw data
		ValidateModel();

		const UE::Anim::DataModel::FEvaluationContext EvaluationContext(ExtractionContext.CurrentTime, DataModelInterface->GetFrameRate(), RetargetSource, MySkeleton->GetRefLocalPoses(RetargetSource), Interpolation);
		DataModelInterface->Evaluate(OutAnimationPoseData, EvaluationContext);

		if ((ExtractionContext.bExtractRootMotion && RootMotionReset.bEnableRootMotion) || RootMotionReset.bForceRootLock)
		{
			RootMotionReset.ResetRootBoneForRootMotion(OutPose[FCompactPoseBoneIndex(0)], RequiredBones);
		}
		return;
	}
#endif

	const FStreamableAnimPlatformData& StreamableAnimData = GetRunningPlatformData();

	const int32 ChunkIndex = GetChunkIndexForTime(StreamableAnimData.Chunks, ExtractionContext.CurrentTime);
	if(ensureMsgf(ChunkIndex != INDEX_NONE, TEXT("Could not get valid chunk with Time %.2f for Streaming Anim %s"), ExtractionContext.CurrentTime, *GetFullName()))
	{
		const FAnimStreamableChunk& CurrentChunk = StreamableAnimData.Chunks[ChunkIndex];
		const IAnimationStreamingManager& StreamingManager = IStreamingManager::Get().GetAnimationStreamingManager();

		const bool bUsingFirstChunk = (ChunkIndex == 0);

		const FCompressedAnimSequence* CurveCompressedDataChunk = StreamingManager.GetLoadedChunk(this, 0, bUsingFirstChunk); //Curve Data stored in chunk 0 till it is properly cropped

		if (!CurveCompressedDataChunk)
		{
#if WITH_EDITOR
			CurveCompressedDataChunk = StreamableAnimData.Chunks[0].CompressedAnimSequence;
#else
			UE_LOGF(LogAnimation, Warning, "Failed to get streamed compressed data Time: %.2f, ChunkIndex:%i, Anim: %ls", ExtractionContext.CurrentTime, 0, *GetFullName());
			return;
#endif
		}

		CurveCompressedDataChunk->CurveCompressionCodec->DecompressCurves(*CurveCompressedDataChunk, OutCurve, ExtractionContext.CurrentTime);

		const FCompressedAnimSequence* CompressedData = bUsingFirstChunk ? CurveCompressedDataChunk : StreamingManager.GetLoadedChunk(this, ChunkIndex, true);

		double ChunkCurrentTime = ExtractionContext.CurrentTime - CurrentChunk.StartTime;
		if (!CompressedData)
		{
#if WITH_EDITOR
			CompressedData = CurrentChunk.CompressedAnimSequence;
#else
			const int32 NumChunks = StreamableAnimData.Chunks.Num();

			int32 FallbackChunkIndex = ChunkIndex;
			while (!CompressedData)
			{
				FallbackChunkIndex = PreviousChunkIndex(FallbackChunkIndex, NumChunks);
				if (FallbackChunkIndex == ChunkIndex)
				{
					//Cannot get fallback
					UE_LOGF(LogAnimation, Warning, "Failed to get ANY streamed compressed data Time: %.2f, ChunkIndex:%i, Anim: %ls", ExtractionContext.CurrentTime, ChunkIndex, *GetFullName());
					return;
				}
				CompressedData = StreamingManager.GetLoadedChunk(this, FallbackChunkIndex, false);
				ChunkCurrentTime = StreamableAnimData.Chunks[FallbackChunkIndex].SequenceLength;
			}

			UE_LOGF(LogAnimation, Warning, "Failed to get streamed compressed data Time: %.2f, ChunkIndex:%i - Using Chunk %i Anim: %ls", ExtractionContext.CurrentTime, ChunkIndex, FallbackChunkIndex, *GetFullName());
#endif
		}

		const int32 NumTracks = CompressedData->CompressedTrackToSkeletonMapTable.Num();
		if (NumTracks == 0)
		{
			return;
		}

		FAnimExtractContext ChunkExtractionContext(ChunkCurrentTime, ExtractionContext.bExtractRootMotion);
		ChunkExtractionContext.BonesRequired = ExtractionContext.BonesRequired;
		ChunkExtractionContext.PoseCurves = ExtractionContext.PoseCurves;

		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Playing Streaming Anim %s Time: %.2f Chunk:%i\n"), *GetName(), ExtractionContext.CurrentTime, ChunkIndex);
		FAnimSequenceDecompressionContext Context(SamplingFrameRate, CurrentChunk.NumFrames, Interpolation, GetFName(), *CompressedData->CompressedDataStructure.Get(), GetSkeleton()->GetRefLocalPoses(), CompressedData->CompressedTrackToSkeletonMapTable, GetSkeleton(), bIsBakedAdditive, AdditiveType);
		UE::Anim::Decompression::DecompressPose(OutPose, *CompressedData, ChunkExtractionContext, Context, RetargetSource, RootMotionReset);
	}
}

void UAnimStreamable::PostLoad()
{
	//Parent PostLoad will ensure that skeleton is fully loaded
	//before we do anything further in PostLoad
	Super::PostLoad();

#if WITH_EDITOR
	if (UAnimSequence* NonConstSeq = const_cast<UAnimSequence*>(ToRawPtr(SourceSequence)))
	{
		NonConstSeq->ConditionalPreload();
		NonConstSeq->ConditionalPostLoad();
	}
	
	const bool bRequiresModelPopulation = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::IntroducingAnimationDataModel;
	const FGuid CurrentGuid = SourceSequence ? SourceSequence->GetDataModel()->GenerateGuid() : FGuid();
	if (SourceSequence && ( CurrentGuid != RawDataGuid || bRequiresModelPopulation))
	{
		InitFrom(SourceSequence);
	}
	else
	{
		RequestCompressedData(); // Grab compressed data for current platform
	}
#else
	IStreamingManager::Get().GetAnimationStreamingManager().AddStreamingAnim(this); // This will be handled by RequestCompressedData in editor builds
#endif
}

void UAnimStreamable::FinishDestroy()
{
	Super::FinishDestroy();

	IStreamingManager::Get().GetAnimationStreamingManager().RemoveStreamingAnim(this);
}

void UAnimStreamable::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
#if WITH_EDITOR
	for (auto& AnimData : StreamableAnimPlatformData)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(AnimData.Value->GetMemorySize());
	}
#else
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetRunningPlatformData().GetMemorySize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(IStreamingManager::Get().GetAnimationStreamingManager().GetMemorySizeForAnim(this));
#endif
}

int32 UAnimStreamable::GetChunkIndexForTime(const TArray<FAnimStreamableChunk>& Chunks, float CurrentTime) const
{
	for (int32 ChunkIndex = 0; ChunkIndex < Chunks.Num(); ++ChunkIndex)
	{
		const float ChunkLength = Chunks[ChunkIndex].SequenceLength;

		if (CurrentTime < ChunkLength)
		{
			return ChunkIndex;
		}
		CurrentTime -= ChunkLength;
	}
	return Chunks.Num() - 1;
}

#if WITH_EDITOR
void UAnimStreamable::InitFrom(const UAnimSequence* InSourceSequence)
{
	Modify();
	SetSkeleton(InSourceSequence->GetSkeleton());
	SourceSequence = InSourceSequence;

	BoneCompressionSettings = InSourceSequence->BoneCompressionSettings;
	CurveCompressionSettings = InSourceSequence->CurveCompressionSettings;
	
	DataModelInterface = StaticDuplicateObject(InSourceSequence->GetDataModelInterface().GetObject(), this);
	VariableFrameStrippingSettings = InSourceSequence->VariableFrameStrippingSettings;

	// Far from ideal (retrieving controller to ensure it matches the DataModelInterface type)
	Controller = DataModelInterface->GetController();
	Controller->SetModel(DataModelInterface);

	Notifies = InSourceSequence->Notifies;
	Controller->SetNumberOfFrames(InSourceSequence->GetDataModel()->GetNumberOfFrames());
	RateScale = InSourceSequence->RateScale;
	Interpolation = InSourceSequence->Interpolation;
	RetargetSource = InSourceSequence->RetargetSource;
	NumberOfKeys = DataModelInterface->GetNumberOfKeys();
	SamplingFrameRate = SourceSequence->GetDataModel()->GetFrameRate();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SequenceLength = SamplingFrameRate.AsSeconds(NumberOfKeys - 1);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Controller->NotifyPopulated();

	bEnableRootMotion = InSourceSequence->bEnableRootMotion;
	RootMotionRootLock = InSourceSequence->RootMotionRootLock;
	bForceRootLock = InSourceSequence->bForceRootLock;
	bUseNormalizedRootMotionScale = InSourceSequence->bUseNormalizedRootMotionScale;

	UpdateRawData();
}

FString GetChunkDDCKey(const FString& BaseKey, uint32 ChunkIndex)
{
	return FString::Printf(TEXT("%s%u"),
		*BaseKey,
		ChunkIndex
	);
}


void UAnimStreamable::RequestCompressedData(const ITargetPlatform* Platform)
{
	check(IsInGameThread());

	bUseRawDataOnly = true;

	USkeleton* CurrentSkeleton = GetSkeleton();
	if (CurrentSkeleton == nullptr)
	{
		return;
	}

	if (GetOutermost() == GetTransientPackage())
	{
		return; // Skip transient animations, they are most likely the leftovers of previous compression attempts.
	}

	if (FPlatformProperties::RequiresCookedData())
	{
		return;
	}
	
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (!TPM)
	{
		return; // No platform manager
	}

	if (!Platform)
	{
		Platform = TPM->GetRunningTargetPlatform();
		check( Platform != nullptr );
	}

	const bool bIsRunningPlatform = Platform->IsRunningPlatform();

	if (bIsRunningPlatform)
	{
		IStreamingManager::Get().GetAnimationStreamingManager().RemoveStreamingAnim(this);
	}

	if (BoneCompressionSettings == nullptr || !BoneCompressionSettings->AreSettingsValid())
	{
		BoneCompressionSettings = FAnimationUtils::GetDefaultAnimationBoneCompressionSettings();
	}

	if (CurveCompressionSettings == nullptr || !CurveCompressionSettings->AreSettingsValid())
	{
		CurveCompressionSettings = FAnimationUtils::GetDefaultAnimationCurveCompressionSettings();
	}

	if (VariableFrameStrippingSettings == nullptr)
	{
		VariableFrameStrippingSettings = FAnimationUtils::GetDefaultVariableFrameStrippingSettings();
	}

	checkf(Platform, TEXT("Failed to specify platform for streamable animation compression"));

	FStreamableAnimPlatformData& PlatformData = GetStreamingAnimPlatformData(Platform);

	if (bIsRunningPlatform)
	{
		RunningAnimPlatformData = &PlatformData;
	}

	PlatformData.Reset();

	float ChunkSizeSeconds = GetChunkSizeSeconds(Platform);

	uint32 NumChunks = 1;
	if (Platform->AllowAudioVisualData() && ChunkSizeSeconds > 0.f) // <= 0.f signifies to not chunk & don't have chunks on a server
	{
		ChunkSizeSeconds = FMath::Max(ChunkSizeSeconds, MINIMUM_CHUNK_SIZE);
		const int32 InitialNumChunks = FMath::FloorToInt(GetPlayLength() / ChunkSizeSeconds);
		NumChunks = FMath::Max(InitialNumChunks, 1);
	}

	const int32 NumFramesToChunk = NumberOfKeys - 1;
	const int32 FramesPerChunk = NumFramesToChunk / NumChunks;

	PlatformData.Chunks.AddDefaulted(NumChunks);

	const FString BaseDDCKey = GetBaseDDCKey(NumChunks, Platform);

	for (uint32 ChunkIndex = 0; ChunkIndex < NumChunks; ++ChunkIndex)
	{
		FString ChunkDDCKey = GetChunkDDCKey(BaseDDCKey, ChunkIndex);

		const bool bLastChunk = (ChunkIndex == (NumChunks - 1));
		const uint32 FrameStart = ChunkIndex * FramesPerChunk;
		const uint32 FrameEnd = bLastChunk ? NumFramesToChunk : (ChunkIndex + 1) * FramesPerChunk;

		RequestCompressedDataForChunk(ChunkDDCKey, PlatformData.Chunks[ChunkIndex], ChunkIndex, FrameStart, FrameEnd, Platform);
	}

	if (bIsRunningPlatform)
	{
		IStreamingManager::Get().GetAnimationStreamingManager().AddStreamingAnim(this);
	}
	//PlatformData.SetSkeletonVirtualBoneGuid(GetSkeleton()->GetVirtualBoneGuid()); //MDW DO THIS
	//PlatformData.bUseRawDataOnly = false; //MDW Need to do something with this?
}

float UAnimStreamable::GetChunkSizeSeconds(const ITargetPlatform* Platform) const
{
	float CVarPlatformChunkSizeSeconds = 1.f;
	if (UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(Platform->IniPlatformName()))
	{
		if (!DeviceProfile->GetConsolidatedCVarValue(ChunkSizeSecondsCVarName, CVarPlatformChunkSizeSeconds))
		{
			CVarPlatformChunkSizeSeconds = MINIMUM_CHUNK_SIZE; //Could not get value so set to Minimum
		}
	}
	
	//UE_LOGF(LogAnimation, Log, "Anim Chunk Size for platform %ls : %.2f\n", *Platform->DisplayName().ToString(), CVarPlatformChunkSizeSeconds);
	return CVarPlatformChunkSizeSeconds;
}

template<typename KeyInType, typename KeyOutType, typename Predicate>
void MakeKeyChunk(const TArray<KeyInType>& SrcKeys, TArray<KeyOutType>& DestKeys, int32 NumberOfKeys, const uint32 FrameStart, const uint32 FrameEnd, Predicate InPredicate)
{
	if (SrcKeys.Num() == 1)
	{
		DestKeys.Add(::Invoke(InPredicate, SrcKeys[0]));
	}
	else
	{
		check(SrcKeys.Num() == NumberOfKeys); // Invalid data otherwise

		DestKeys.Reset((FrameEnd - FrameStart) + 1);
		for (uint32 FrameIndex = FrameStart; FrameIndex <= FrameEnd; ++FrameIndex)
		{
			DestKeys.Add(::Invoke(InPredicate, SrcKeys[FrameIndex]));
		}
	}
}

void UAnimStreamable::RequestCompressedDataForChunk(const FString& ChunkDDCKey, FAnimStreamableChunk& Chunk, const int32 ChunkIndex, const uint32 FrameStart, const uint32 FrameEnd, const ITargetPlatform* Platform)
{
	// Need to unify with Anim Sequence!

	TArray<uint8> OutData;
	{
		COOK_STAT(auto Timer = AnimStreamableCookStats::UsageStats.TimeSyncWork());

		bool bNeedToCleanUpAnimCompressor = true;
		FDerivedDataAnimationCompression* AnimCompressor = new FDerivedDataAnimationCompression(TEXT("StreamAnim"), ChunkDDCKey);

		const FString FinalDDCKey = FDerivedDataCacheInterface::BuildCacheKey(AnimCompressor->GetPluginName(), AnimCompressor->GetVersionString(), *AnimCompressor->GetPluginSpecificCacheKeySuffix());

		// For debugging DDC/Compression issues		
		bool bSkipDDC = false;

		const int32 ChunkNumFrames = FrameEnd - FrameStart;
		Chunk.StartTime = SamplingFrameRate.AsSeconds(static_cast<int32>(FrameStart));
		Chunk.SequenceLength = SamplingFrameRate.AsSeconds(ChunkNumFrames);
		Chunk.NumFrames = ChunkNumFrames;

		if (!bSkipDDC && GetDerivedDataCacheRef().GetSynchronous(*FinalDDCKey, OutData, GetPathName()))
		{
			COOK_STAT(Timer.AddHit(OutData.Num()));
		}
		else
		{
			FCompressibleAnimRef CompressibleData = MakeShared<FCompressibleAnimData, ESPMode::ThreadSafe>(BoneCompressionSettings, CurveCompressionSettings, GetSkeleton(), Interpolation, Chunk.SequenceLength, ChunkNumFrames+1, Platform);

			TArray<FName> TrackNames;
			DataModelInterface->GetBoneTrackNames(TrackNames);
			
			CompressibleData->TrackToSkeletonMapTable.Empty();

			TArray<FTransform> BoneTransforms;
			for (const FName& TrackName : TrackNames)
			{
				const int32 BoneIndex = GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(TrackName);
				if (BoneIndex != INDEX_NONE)
				{
					FRawAnimSequenceTrack& DestTrack = CompressibleData->RawAnimationData.AddDefaulted_GetRef();
					
					BoneTransforms.Reset();
					DataModelInterface->GetBoneTrackTransforms(TrackName, BoneTransforms);
					

					MakeKeyChunk(BoneTransforms, DestTrack.PosKeys, NumberOfKeys, FrameStart, FrameEnd, [](const FTransform& BoneTransform) -> FVector3f
					{
						return FVector3f(BoneTransform.GetLocation());
					});
					
					MakeKeyChunk(BoneTransforms, DestTrack.RotKeys, NumberOfKeys, FrameStart, FrameEnd, [](const FTransform& BoneTransform) -> FQuat4f
					{
						return FQuat4f(BoneTransform.GetRotation());
					});
					
					MakeKeyChunk(BoneTransforms, DestTrack.ScaleKeys, NumberOfKeys, FrameStart, FrameEnd, [](const FTransform& BoneTransform) -> FVector3f
					{
						return FVector3f(BoneTransform.GetScale3D());
					});

					CompressibleData->TrackToSkeletonMapTable.Add(BoneIndex);
				}
			}

			if (FrameStart == 0)
			{
				//Crop curve logic broken, for the moment store curve data in always loaded chunk 0
				CompressibleData->RawFloatCurves = DataModelInterface->GetFloatCurves();
			}

			// Data is manually populated above, so skip AnimationCompression builds/fetch data steps
			CompressibleData->bDataFetched = true;
			AnimCompressor->SetCompressibleData(CompressibleData);

			if (bSkipDDC)
			{
				AnimCompressor->Build(OutData);
				COOK_STAT(Timer.AddMiss(OutData.Num()));
			}
			else if (AnimCompressor->CanBuild())
			{
				bNeedToCleanUpAnimCompressor = false; // GetSynchronous will handle this
				bool bBuilt = false;
				const bool bSuccess = GetDerivedDataCacheRef().GetSynchronous(AnimCompressor, OutData, &bBuilt);
				COOK_STAT(Timer.AddHitOrMiss(!bSuccess || bBuilt ? FCookStats::CallStats::EHitOrMiss::Miss : FCookStats::CallStats::EHitOrMiss::Hit, OutData.Num()));
			}
			else
			{
				COOK_STAT(Timer.TrackCyclesOnly());
			}
		}

		if (bNeedToCleanUpAnimCompressor)
		{
			delete AnimCompressor; // Would really like to do auto mem management but GetDerivedDataCacheRef().GetSynchronous expects a point it can delete, every
			AnimCompressor = nullptr;
		}
	}

	check(OutData.Num() > 0); // Should always have "something"
	{
		FMemoryReader MemAr(OutData);

		if (!Chunk.CompressedAnimSequence)
		{
			Chunk.CompressedAnimSequence = new FCompressedAnimSequence();
		}
		Chunk.CompressedAnimSequence->SerializeCompressedData(MemAr, true, this, this->GetSkeleton(), BoneCompressionSettings, CurveCompressionSettings);
	}
}

void UAnimStreamable::UpdateRawData()
{
	if (IsDataModelValid())
	{
		RawDataGuid = DataModelInterface->GenerateGuid();
		RequestCompressedData();
	}
}

FString UAnimStreamable::GetBaseDDCKey(uint32 NumChunks, const ITargetPlatform* TargetPlatform) const
{
	//Make up our content key consisting of:
	//  * Streaming Anim Chunk logic version
	//	* Our raw data GUID
	//	* Our skeleton GUID: If our skeleton changes our compressed data may now be stale
	//  * Our skeletons virtual bone guid
	//	* Compression Settings
	//	* Curve compression settings
	//  * Variable frame stripping settings

	FArcToHexString ArcToHexString;

	ArcToHexString.Ar << NumChunks;
	BoneCompressionSettings->PopulateDDCKey(UE::Anim::Compression::FAnimDDCKeyArgs(*this, TargetPlatform), ArcToHexString.Ar);
	CurveCompressionSettings->PopulateDDCKey(ArcToHexString.Ar);
	VariableFrameStrippingSettings->PopulateDDCKey(UE::Anim::Compression::FAnimDDCKeyArgs(*this, TargetPlatform), ArcToHexString.Ar);

	FString Ret = FString::Printf(TEXT("%s%s%s%s_%s"),
		StreamingAnimChunkVersion,
		*RawDataGuid.ToString(),
		*GetSkeleton()->GetGuid().ToString(),
		*GetSkeleton()->GetVirtualBoneGuid().ToString(),
		*ArcToHexString.MakeString()
	);

	return Ret;
}

#endif

FStreamableAnimPlatformData& UAnimStreamable::GetStreamingAnimPlatformData(const ITargetPlatform* Platform)
{
#if WITH_EDITOR
	FStreamableAnimPlatformData** Data = StreamableAnimPlatformData.Find(Platform);
	if (!Data)
	{
		return *StreamableAnimPlatformData.Add(Platform, new FStreamableAnimPlatformData());
	}
	checkf(*Data, TEXT("StreamableAnimPlatformData contains nullptr!"));
	return **Data;
#else
	return RunningAnimPlatformData;
#endif
}
