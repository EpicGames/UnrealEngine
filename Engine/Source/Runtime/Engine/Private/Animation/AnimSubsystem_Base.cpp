// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSubsystem_Base.h"
#include "Animation/AnimSubsystem.h"
#include "Animation/AnimClassInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSubsystem_Base)

void FAnimSubsystem_Base::OnPostLoadDefaults(FAnimSubsystemPostLoadDefaultsContext& InContext)
{
	PatchValueHandlers(InContext.DefaultAnimInstance->GetClass());
}

void FAnimSubsystem_Base::PatchValueHandlers(UClass* InClass)
{
	FScopeLock Lock(&ValueHandlersCriticalSection);
	
	ExposedValueHandlers.Reset();

	void* SparseClassData = const_cast<void*>(InClass->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull));
	check(SparseClassData);
	for (TFieldIterator<FProperty> ItParam(InClass->GetSparseClassDataStruct()); ItParam; ++ItParam)
	{
		if (FStructProperty* StructProperty = CastField<FStructProperty>(*ItParam))
		{
			if (StructProperty->Struct->IsChildOf(FAnimNodeExposedValueHandler::StaticStruct()))
			{
				FExposedValueHandler& NewHandler = ExposedValueHandlers.AddDefaulted_GetRef();
				NewHandler.HandlerStruct = StructProperty->Struct;
				NewHandler.Handler = StructProperty->ContainerPtrToValuePtr<FAnimNodeExposedValueHandler>(SparseClassData);
				NewHandler.Handler->Initialize(InClass);
			}
		}
	}
}

// Unlike the copy assignment operator, the copy constructor doesn't take a lock (it is not called during UClass::CreateSparseClassData)
FAnimSubsystem_Base::FAnimSubsystem_Base(const FAnimSubsystem_Base& Other)
	: ExposedValueHandlers(Other.ExposedValueHandlers)
{
}

FAnimSubsystem_Base& FAnimSubsystem_Base::operator =(const FAnimSubsystem_Base& Other)
{
	// Need to perform a lock as copying can race with PatchValueHandlers in async loading thread-enabled builds
	FScopeLock Lock(&ValueHandlersCriticalSection);
	FScopeLock OtherLock(&Other.ValueHandlersCriticalSection);

	ExposedValueHandlers = Other.ExposedValueHandlers;

	return *this;
}
