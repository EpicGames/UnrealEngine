// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSubsystem_NodeRelevancy.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSubsystem_NodeRelevancy)

float FAnimNodeRelevancyStatus::GetCurrentWeight() const
{
	return CurrentWeight;
}

float FAnimNodeRelevancyStatus::GetPreviousWeight() const
{
	return PreviousWeight;
}

bool FAnimNodeRelevancyStatus::IsStartingBlendingOut() const
{
	return FAnimWeight::IsFullWeight(PreviousWeight) && !FAnimWeight::IsFullWeight(CurrentWeight);
}

bool FAnimNodeRelevancyStatus::IsStartingBlendingIn() const
{
	return !FAnimWeight::IsRelevant(PreviousWeight) && FAnimWeight::IsRelevant(CurrentWeight);
}

bool FAnimNodeRelevancyStatus::IsFinishingBlendingIn() const
{
	return !FAnimWeight::IsFullWeight(PreviousWeight) && FAnimWeight::IsFullWeight(CurrentWeight);
}

bool FAnimNodeRelevancyStatus::IsFinishingBlendingOut() const
{
	return FAnimWeight::IsRelevant(PreviousWeight) && !FAnimWeight::IsRelevant(CurrentWeight);
}

bool FAnimNodeRelevancyStatus::HasFullyBlendedIn() const
{
	return FAnimWeight::IsFullWeight(PreviousWeight) && FAnimWeight::IsFullWeight(CurrentWeight);
}

bool FAnimNodeRelevancyStatus::HasFullyBlendedOut() const
{
	return !FAnimWeight::IsRelevant(PreviousWeight) && !FAnimWeight::IsRelevant(CurrentWeight);
}

void FAnimSubsystemInstance_NodeRelevancy::Initialize_WorkerThread()
{
	NodeTrackers.Empty();
}

FAnimNodeRelevancyStatus FAnimSubsystemInstance_NodeRelevancy::UpdateNodeRelevancy(const FAnimationUpdateContext& InContext, const FAnimNode_Base& InNode)
{
	FTracker& Tracker = NodeTrackers.FindOrAdd(&InNode);
	
	if(!Tracker.Counter.HasEverBeenUpdated() || !Tracker.Counter.WasSynchronizedCounter(InContext.AnimInstanceProxy->GetUpdateCounter()))
	{
		Tracker.Status.CurrentWeight = 0.0f;
	}

	if(!Tracker.Counter.IsSynchronized_All(InContext.AnimInstanceProxy->GetUpdateCounter()))
	{
		Tracker.Counter.SynchronizeWith(InContext.AnimInstanceProxy->GetUpdateCounter());
		Tracker.Status.PreviousWeight = Tracker.Status.CurrentWeight;
		Tracker.Status.CurrentWeight = InContext.GetFinalBlendWeight();
	}
	
	return Tracker.Status;
}

FAnimNodeRelevancyStatus FAnimSubsystemInstance_NodeRelevancy::GetNodeRelevancy(const FAnimNode_Base& InNode) const
{
	if(const FTracker* Tracker = NodeTrackers.Find(&InNode))
	{
		return Tracker->Status;
	}

	return FAnimNodeRelevancyStatus();
}

EAnimNodeInitializationStatus FAnimSubsystemInstance_NodeRelevancy::UpdateNodeInitializationStatus(const FAnimationUpdateContext& InContext, const FAnimNode_Base& InNode)
{
	EAnimNodeInitializationStatus& Status = NodeInitTrackers.FindOrAdd(&InNode);

	switch(Status)
	{
	case EAnimNodeInitializationStatus::NotUpdated:
		Status = EAnimNodeInitializationStatus::InitialUpdate;
		break;
	default:
	case EAnimNodeInitializationStatus::InitialUpdate:
		Status = EAnimNodeInitializationStatus::Updated;
		break;
	}

	return Status;
}

EAnimNodeInitializationStatus FAnimSubsystemInstance_NodeRelevancy::GetNodeInitializationStatus(const FAnimNode_Base& InNode) const
{
	return NodeInitTrackers.FindRef(&InNode);
}
