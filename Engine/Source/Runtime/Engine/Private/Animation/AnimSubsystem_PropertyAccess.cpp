// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSubsystem_PropertyAccess.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSubsystem_PropertyAccess)

void FAnimSubsystem_PropertyAccess::OnPreUpdate_GameThread(FAnimSubsystemUpdateContext& InContext) const
{
	// Process internal batched property copies
	PropertyAccess::ProcessCopies(InContext.AnimInstance, Library, PropertyAccess::FCopyBatchId((int32)EAnimPropertyAccessCallSite::GameThread_Batched_PreEventGraph));
}

void FAnimSubsystem_PropertyAccess::OnPostUpdate_GameThread(FAnimSubsystemUpdateContext& InContext) const
{
	// Process internal batched property copies
	PropertyAccess::ProcessCopies(InContext.AnimInstance, Library, PropertyAccess::FCopyBatchId((int32)EAnimPropertyAccessCallSite::GameThread_Batched_PostEventGraph));
}

void FAnimSubsystem_PropertyAccess::OnPreUpdate_WorkerThread(FAnimSubsystemParallelUpdateContext& InContext) const
{
	// Process internal batched property copies
	PropertyAccess::ProcessCopies(InContext.Proxy.GetAnimInstanceObject(), Library, PropertyAccess::FCopyBatchId((int32)EAnimPropertyAccessCallSite::WorkerThread_Batched_PreEventGraph));
}

void FAnimSubsystem_PropertyAccess::OnPostUpdate_WorkerThread(FAnimSubsystemParallelUpdateContext& InContext) const
{
	// Process internal batched property copies
	PropertyAccess::ProcessCopies(InContext.Proxy.GetAnimInstanceObject(), Library, PropertyAccess::FCopyBatchId((int32)EAnimPropertyAccessCallSite::WorkerThread_Batched_PostEventGraph));
}

void FAnimSubsystem_PropertyAccess::OnPostLoad(FAnimSubsystemPostLoadContext& InContext)
{
	// Patch the library on load to fixup property offsets
	PropertyAccess::PatchPropertyOffsets(Library);
}

#if WITH_EDITORONLY_DATA
void FAnimSubsystem_PropertyAccess::OnLink(FAnimSubsystemLinkContext& InContext)
{
	// Patch the library on load to fixup property offsets
	PropertyAccess::PatchPropertyOffsets(Library);
}
#endif
