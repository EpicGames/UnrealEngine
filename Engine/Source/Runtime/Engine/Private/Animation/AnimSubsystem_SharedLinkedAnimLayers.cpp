// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSubsystem_SharedLinkedAnimLayers.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "ObjectTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSubsystem_SharedLinkedAnimLayers)

namespace SharedLinkedAnimLayersConsoleCommands
{
	static int32 MarkLayerAsGarbageOnUninitialize = 0;
	static FAutoConsoleVariableRef CVarMarkLayerAsGarbageOnUninitialize(
		TEXT("a.MarkLayerAsGarbageOnUninitialize"), MarkLayerAsGarbageOnUninitialize,
		TEXT("Whether to mark the layers as garbage after uinitializing them."),
		ECVF_Default);
}

#if LINKEDANIMLAYERSDATA_INTEGRITYCHECKS
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimNode_LinkedAnimLayer.h"

namespace
{
	// Check that function is either linked (!bIsFree) or unlinked (bIsFree)
	void CheckLayerDataIntegrity(const UAnimInstance* AnimInstance, FName FunctionName, bool bIsFree)
	{
		bool bFound = false;

		TArray<const UAnimInstance*> AllAnimInstances;
		const USkeletalMeshComponent* SkelMesh = AnimInstance->GetSkelMeshComponent();
		AllAnimInstances.Add(SkelMesh->GetAnimInstance());
		AllAnimInstances.Append(SkelMesh->GetLinkedAnimInstances());
		for (const UAnimInstance* LinkedAnimInstance : AllAnimInstances)
		{
			const IAnimClassInterface* const NewLinkedInstanceClass = IAnimClassInterface::GetFromClass(LinkedAnimInstance->GetClass());
			for (const FStructProperty* const LayerNodeProperty : NewLinkedInstanceClass->GetLinkedAnimLayerNodeProperties())
			{
				const FAnimNode_LinkedAnimLayer* const LinkedAnimLayerNode = LayerNodeProperty->ContainerPtrToValuePtr<const FAnimNode_LinkedAnimLayer>(LinkedAnimInstance);
				if (LinkedAnimLayerNode->GetDynamicLinkFunctionName() == FunctionName)
				{
					if (LinkedAnimLayerNode->GetTargetInstance<const UAnimInstance>() == AnimInstance)
					{
						// Function should be free to link but isn't
						if (bIsFree)
						{
							check(0);
						}
						else
						{
							// Function shouldn't be linked more than once
							check(!bFound);
							bFound = true;
						}
					}
				}
			}
		}
		// Function is either found or free
		check(bIsFree || bFound);
	}
}

#endif // LINKEDANIMLAYERSDATA_INTEGRITYCHECKS

void FLinkedAnimLayerInstanceData::AddLinkedFunction(FName Function, UAnimInstance* AnimInstance)
{
#if LINKEDANIMLAYERSDATA_INTEGRITYCHECKS
	CheckLayerDataIntegrity(Instance, Function, true);
#endif
	check(!LinkedFunctions.Contains(Function));
	LinkedFunctions.Add(Function, AnimInstance);
}

void FLinkedAnimLayerInstanceData::RemoveLinkedFunction(FName Function)
{
#if LINKEDANIMLAYERSDATA_INTEGRITYCHECKS
	CheckLayerDataIntegrity(Instance, Function, true);
#endif
	check(LinkedFunctions.Contains(Function));
	LinkedFunctions.Remove(Function);
}

FLinkedAnimLayerInstanceData* FLinkedAnimLayerClassData::FindInstanceData(const UAnimInstance* AnimInstance)
{
	return InstancesData.FindByPredicate([AnimInstance](FLinkedAnimLayerInstanceData& InstanceData) {return InstanceData.Instance == AnimInstance; });
}

UAnimInstance* FLinkedAnimLayerClassData::FindOrAddInstanceForLinking(UAnimInstance* OwningInstance, FName Function, bool& bIsNewInstance)
{
	USkeletalMeshComponent* Mesh = OwningInstance->GetSkelMeshComponent();

	for (FLinkedAnimLayerInstanceData& LayerInstanceData : InstancesData)
	{
		// Check if function is already linked
		if (!LayerInstanceData.GetLinkedFunctions().Contains(Function))
		{
			// Re-add persistent instance of first function re-bind
			if (LayerInstanceData.IsPersistent() && LayerInstanceData.GetLinkedFunctions().Num() == 0)
			{
				// Make sure the bones to update are up to date with LOD changes / bone visibility / cosmetics, etc.
				LayerInstanceData.Instance->RecalcRequiredBones();
				check(!Mesh->GetLinkedAnimInstances().Contains(LayerInstanceData.Instance));
				Mesh->GetLinkedAnimInstances().Add(LayerInstanceData.Instance);
			}

			bIsNewInstance = false;
			LayerInstanceData.AddLinkedFunction(Function, LayerInstanceData.Instance);
			// not linked, use this instance
			return LayerInstanceData.Instance;
		}
	}

	// Create a new object
	bIsNewInstance = true;
	UAnimInstance* NewAnimInstance = NewObject<UAnimInstance>(Mesh, Class);
	NewAnimInstance->bCreatedByLinkedAnimGraph = true;
	NewAnimInstance->InitializeAnimation();

	if(Mesh->HasBegunPlay())
	{
		NewAnimInstance->NativeBeginPlay();
		NewAnimInstance->BlueprintBeginPlay();
	}
	
	FLinkedAnimLayerInstanceData& NewInstanceData = AddInstance(NewAnimInstance);
	NewInstanceData.AddLinkedFunction(Function, NewAnimInstance);
	OwningInstance->GetSkelMeshComponent()->GetLinkedAnimInstances().Add(NewAnimInstance);

	return NewAnimInstance;
}

FLinkedAnimLayerInstanceData& FLinkedAnimLayerClassData::AddInstance(UAnimInstance* AnimInstance)
{
	// First instance we create for persistent layer is marked has persistent
	InstancesData.Push(FLinkedAnimLayerInstanceData(AnimInstance, bIsPersistent && InstancesData.Num() == 0));
	return InstancesData.Last();
}

void FLinkedAnimLayerClassData::RemoveLinkedFunction(UAnimInstance* AnimInstance, FName Function)
{
	if (FLinkedAnimLayerInstanceData* InstanceData = FindInstanceData(AnimInstance))
	{
		InstanceData->RemoveLinkedFunction(Function);
		if (InstanceData->GetLinkedFunctions().Num() == 0)
		{
			if (USkeletalMeshComponent* Mesh = InstanceData->Instance->GetSkelMeshComponent())
			{
				Mesh->GetLinkedAnimInstances().Remove(AnimInstance);
			}
			if (!InstanceData->IsPersistent())
			{
				RemoveInstance(AnimInstance);
			}
		}
	}
}

void FLinkedAnimLayerClassData::RemoveInstance(UAnimInstance* AnimInstance)
{
	for (int i = 0; i < InstancesData.Num(); ++i)
	{
		if (InstancesData[i].Instance == AnimInstance)
		{
			const FLinkedAnimLayerInstanceData& InstanceData = InstancesData[i];

			// If we have no function linked, the instance should never be part of the skeletal mesh component
			check(!InstanceData.Instance->GetSkelMeshComponent() || !InstanceData.Instance->GetSkelMeshComponent()->GetLinkedAnimInstances().Contains(InstanceData.Instance));
			// Make sure no functions are still linked
			check(InstanceData.GetLinkedFunctions().Num() == 0);
			check(!InstanceData.IsPersistent());

			// Since UninitializeAnimation can make calls to the shared layer system via self layer nodes, cleanup before Uninitializing to prevent unnecessary checks
			InstancesData.RemoveAt(i);

			AnimInstance->UninitializeAnimation();
			TRACE_OBJECT_LIFETIME_END(AnimInstance);
			if (SharedLinkedAnimLayersConsoleCommands::MarkLayerAsGarbageOnUninitialize)
			{
				AnimInstance->MarkAsGarbage();
			}
			return;
		}
	}
	check(0);
}

void FLinkedAnimLayerClassData::SetPersistence(bool bInIsPersistent)
{
	if (bInIsPersistent != IsPersistent())
	{
		bIsPersistent = bInIsPersistent;
		// If persistence is added we mark the current linked instance, if there is one, as persistent
		if (bIsPersistent)
		{
			if (InstancesData.Num())
			{
				InstancesData[0].SetPersistence(true);
			}
		}
		// If we remove persistence however make sure any currently unlinked instance is correctly deleted
		else // if (!bIsPersistent)
		{
			int32 InstanceIndex = InstancesData.IndexOfByPredicate([](const FLinkedAnimLayerInstanceData& InstanceData) {return InstanceData.IsPersistent(); });
			if (InstanceIndex != INDEX_NONE)
			{
				FLinkedAnimLayerInstanceData& InstanceData = InstancesData[InstanceIndex];
				InstanceData.SetPersistence(false);
				if (InstanceData.GetLinkedFunctions().Num() == 0)
				{
					RemoveInstance(InstanceData.Instance);
				}
			}
		}
	}
}
FAnimSubsystem_SharedLinkedAnimLayers* FAnimSubsystem_SharedLinkedAnimLayers::GetFromMesh(USkeletalMeshComponent* SkelMesh)
{
#if WITH_EDITOR
	if (GIsReinstancing)
	{
		return nullptr;
	}
#endif
	check(SkelMesh);

	// In some cases we have a PostProcessAnimInstance but no AnimScriptInstance
	if (SkelMesh->GetAnimInstance())
	{
		return SkelMesh->GetAnimInstance()->FindSubsystem<FAnimSubsystem_SharedLinkedAnimLayers>();
	}
	return nullptr;
}

void FAnimSubsystem_SharedLinkedAnimLayers::Reset()
{
	PersistentClasses.Empty();
	ClassesData.Empty(ClassesData.Num());
}

FLinkedAnimLayerInstanceData* FAnimSubsystem_SharedLinkedAnimLayers::FindInstanceData(const UAnimInstance* AnimInstance)
{
	if (FLinkedAnimLayerClassData* ClassData = FindClassData(AnimInstance->GetClass()))
	{
		return ClassData->FindInstanceData(AnimInstance);
	}
	return nullptr;
}

FLinkedAnimLayerClassData* FAnimSubsystem_SharedLinkedAnimLayers::FindClassData(TSubclassOf<UAnimInstance> AnimClass) 
{
	return ClassesData.FindByPredicate([AnimClass](const FLinkedAnimLayerClassData& ClassData) {return ClassData.GetClass() == AnimClass; });
}

FLinkedAnimLayerClassData& FAnimSubsystem_SharedLinkedAnimLayers::FindOrAddClassData(TSubclassOf<UAnimInstance> AnimClass)
{
	if (FLinkedAnimLayerClassData* Result = FindClassData(AnimClass))
	{
		return *Result;
	}
	bool bIsPersistent = PersistentClasses.Find(AnimClass) != INDEX_NONE;
	ClassesData.Add(FLinkedAnimLayerClassData(AnimClass, bIsPersistent));
	return ClassesData.Last();
};

void FAnimSubsystem_SharedLinkedAnimLayers::RemovePersistentAnimLayerClass(TSubclassOf<UAnimInstance> AnimInstanceClass)
{
	// When a class loses its persistency, make sure to clean it up if it's already unlinked
	int32 ClassIndex = ClassesData.IndexOfByPredicate([AnimInstanceClass](const FLinkedAnimLayerClassData& ClassData) {return ClassData.GetClass() == AnimInstanceClass; });
	if (ClassIndex != INDEX_NONE)
	{
		FLinkedAnimLayerClassData& ClassData = ClassesData[ClassIndex];
		ClassData.SetPersistence(false);
	}
	PersistentClasses.Remove(AnimInstanceClass);

}

UAnimInstance* FAnimSubsystem_SharedLinkedAnimLayers::AddLinkedFunction(UAnimInstance* OwningInstance, TSubclassOf<UAnimInstance> AnimClass, FName Function, bool& bIsNewInstance)
{
	FLinkedAnimLayerClassData& ClassData = FindOrAddClassData(AnimClass);
	return ClassData.FindOrAddInstanceForLinking(OwningInstance, Function, bIsNewInstance);
}

void FAnimSubsystem_SharedLinkedAnimLayers::RemoveLinkedFunction(UAnimInstance* AnimInstance, FName Function)
{
	int32 ClassIndex = ClassesData.IndexOfByPredicate([AnimInstance](const FLinkedAnimLayerClassData& ClassData) {return ClassData.GetClass() == AnimInstance->GetClass(); });
	if (ClassIndex != INDEX_NONE)
	{
		FLinkedAnimLayerClassData& ClassData = ClassesData[ClassIndex];
		ClassData.RemoveLinkedFunction(AnimInstance, Function);
		if (ClassData.GetInstancesData().Num() == 0)
		{
			ClassesData.RemoveAt(ClassIndex);
		}
	}
}
