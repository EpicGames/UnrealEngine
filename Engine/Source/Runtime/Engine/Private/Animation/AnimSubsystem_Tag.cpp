// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSubsystem_Tag.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSubsystem_Tag)

int32 FAnimSubsystem_Tag::FindNodeIndexByTag(FName InTag) const
{
	if(const int32* NodeIndexPtr = NodeIndices.Find(InTag))
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClass->GetAnimNodeProperties();
		// As the index is patched during compilation, it needs to be reversed here
		int32 ReverseIndex = AnimNodeProperties.Num() - 1 - *NodeIndexPtr;
		return ReverseIndex;
	}

	return INDEX_NONE;
}

FAnimNode_Base* FAnimSubsystem_Tag::FindNodeByTag_Internal(FName InTag, UAnimInstance* InInstance, UScriptStruct* InNodeType) const
{
	ensure(InInstance->GetClass()->IsChildOf(IAnimClassInterface::GetActualAnimClass(AnimClass)));

	if(const int32* NodeIndexPtr = NodeIndices.Find(InTag))
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimClass->GetAnimNodeProperties();
		// As the index is patched during compilation, it needs to be reversed here
		int32 ReverseIndex = AnimNodeProperties.Num() - 1 - *NodeIndexPtr;
		
		check(AnimNodeProperties.IsValidIndex(ReverseIndex));
		FStructProperty* NodeProperty = AnimClass->GetAnimNodeProperties()[ReverseIndex];
		if(NodeProperty->Struct->IsChildOf(InNodeType))
		{
			return NodeProperty->ContainerPtrToValuePtr<FAnimNode_Base>(InInstance);
		}
	}

	return nullptr;
}

void FAnimSubsystem_Tag::OnPostLoadDefaults(FAnimSubsystemPostLoadDefaultsContext& InContext)
{
	AnimClass = IAnimClassInterface::GetFromClass(InContext.DefaultAnimInstance->GetClass());
}
