// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSync.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "AnimationUtils.h"
#include "Animation/MirrorDataTable.h"
#include "AnimationRuntime.h"
#include "Misc/StringFormatArg.h"
#include "Stats/StatsHierarchical.h"

namespace UE { namespace Anim {

const FName FAnimSync::Attribute("Sync");

void FAnimSync::Reset()
{
	TArray<FAnimTickRecord>& UngroupedActivePlayers = UngroupedActivePlayerArrays[GetSyncGroupWriteIndex()];
	UngroupedActivePlayers.Reset();

	FSyncGroupMap& SyncGroups = SyncGroupMaps[GetSyncGroupWriteIndex()];
	for (auto& SyncGroupPair : SyncGroups)
	{
		SyncGroupPair.Value.Reset();
	}
}

void FAnimSync::ResetAll()
{
	for(TArray<FAnimTickRecord>& UngroupedActivePlayers : UngroupedActivePlayerArrays)
	{
		UngroupedActivePlayers.Reset();
	}

	for(FSyncGroupMap& SyncGroupMap : SyncGroupMaps)
	{
		for (auto& SyncGroupPair : SyncGroupMap)
		{
			SyncGroupPair.Value.Reset();
		}
	}

	MirrorDataTable = nullptr;
	SyncGroupWriteIndex = 0;
}

void FAnimSync::AddTickRecord(const FAnimTickRecord& InTickRecord, const FAnimSyncParams& InSyncParams)
{
	if (InSyncParams.GroupName != NAME_None)
	{
		// Get specified sync group for the animation instance. 
		FSyncGroupMap& SyncGroupMap = SyncGroupMaps[GetSyncGroupWriteIndex()];
		FAnimGroupInstance& SyncGroupInstance = SyncGroupMap.FindOrAdd(InSyncParams.GroupName);

		// Add animation instance's tick record to the sync group. 
		SyncGroupInstance.ActivePlayers.Add(InTickRecord);
		SyncGroupInstance.ActivePlayers.Top().MirrorDataTable = MirrorDataTable;
		SyncGroupInstance.ActivePlayers.Top().bOverridePositionWhenJoiningSyncGroupAsLeader = InSyncParams.bOverridePositionWhenJoiningSyncGroupAsLeader;
		SyncGroupInstance.ActivePlayers.Top().bIsExclusiveLeader = InSyncParams.Role == EAnimGroupRole::ExclusiveAlwaysLeader;

		// Set leader score for the tick record we just added, and ensure there is only one montage per group.
		// CanBeLeader or TransitionLeader Score (BlendWeight) < Always Leader Score (2.0) < Montage Leader Score (3.0).
		SyncGroupInstance.TestTickRecordForLeadership(InSyncParams.Role);
	}
	else
	{
		UngroupedActivePlayerArrays[GetSyncGroupWriteIndex()].Add(InTickRecord);
		UngroupedActivePlayerArrays[GetSyncGroupWriteIndex()].Top().MirrorDataTable = MirrorDataTable;
	}
}

void FAnimSync::SetMirror(const UMirrorDataTable* MirrorTable)
{
	MirrorDataTable = MirrorTable;
}

void FAnimSync::TickAssetPlayerInstances(FAnimInstanceProxy& InProxy, float InDeltaSeconds)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	SCOPE_CYCLE_COUNTER(STAT_TickAssetPlayerInstances);

	// Helper function to accumulate the root motion of a tick record.
	auto AccumulateRootMotion = [&InProxy](FAnimTickRecord& TickRecord, FAnimAssetTickContext& TickContext)
	{
		if (TickRecord.MirrorDataTable)
		{
			FTransform InTransform = TickContext.RootMotionMovementParams.GetRootMotionTransform();
			FVector T = InTransform.GetTranslation();
			T = FAnimationRuntime::MirrorVector(T, TickRecord.MirrorDataTable->MirrorAxis);

			FQuat Q = InTransform.GetRotation();
			Q = FAnimationRuntime::MirrorQuat(Q, TickRecord.MirrorDataTable->MirrorAxis);

			FVector S = InTransform.GetScale3D();
			FTransform MirroredTransform = FTransform(Q, T, S);
			InProxy.ExtractedRootMotion.AccumulateWithBlend(MirroredTransform, TickRecord.GetRootMotionWeight());
		}
		else
		{
			InProxy.ExtractedRootMotion.AccumulateWithBlend(TickContext.RootMotionMovementParams, TickRecord.GetRootMotionWeight());
		}
	};

	const ERootMotionMode::Type RootMotionMode = InProxy.GetRootMotionMode();

	// Tick grouped animation instances.
	{
		FSyncGroupMap& SyncGroupMap = SyncGroupMaps[GetSyncGroupWriteIndex()];
		const FSyncGroupMap& PreviousSyncGroupMap = SyncGroupMaps[GetSyncGroupReadIndex()];
		
		// Handle animation players inside group instances.
		for (TTuple<FName, FAnimGroupInstance>& SyncGroupPair : SyncGroupMap)
		{
			FAnimGroupInstance& SyncGroup = SyncGroupPair.Value;

			if (SyncGroup.ActivePlayers.Num() > 0)
			{
				const FAnimGroupInstance* PreviousGroup = PreviousSyncGroupMap.Find(SyncGroupPair.Key);

				// Prepare sync group.
				// This is where our asset players in the group are sorted based on a leader score, and tick record are reset if needed.
				// Additionally, any markers that are not common to all animations in group are removed.
				SyncGroup.Prepare(PreviousGroup);

				UE_LOGF(LogAnimMarkerSync, Log, "Ticking Group [%ls] GroupLeader [%d]", *SyncGroupPair.Key.ToString(), SyncGroup.GroupLeaderIndex);

				// Determine if we have a single animation context.
				const bool bOnlyOneAnimationInGroup = SyncGroup.ActivePlayers.Num() == 1;

				// Animation context for group. (Modified by leader, read by followers)
				FAnimAssetTickContext TickContext(InDeltaSeconds, RootMotionMode, bOnlyOneAnimationInGroup, SyncGroup.ValidMarkers);

				// Initialize group context using previous update's group information.
				if (PreviousGroup)
				{
					// Initialize the anim position ratio from the previous frame's final anim position ratio in case we're not using marker based sync.
					TickContext.SetPreviousAnimationPositionRatio(PreviousGroup->AnimLengthRatio);
					TickContext.SetAnimationPositionRatio(PreviousGroup->AnimLengthRatio);
					
					// Continue from where previous group marker-based sync left off, use its end sync position as our new start sync position. 
					const FMarkerSyncAnimPosition& EndPosition = PreviousGroup->MarkerTickContext.GetMarkerSyncEndPosition();
					if (EndPosition.IsValid() && (EndPosition.PreviousMarkerName == NAME_None || SyncGroup.ValidMarkers.Contains(EndPosition.PreviousMarkerName))
					                          && (EndPosition.NextMarkerName == NAME_None || SyncGroup.ValidMarkers.Contains(EndPosition.NextMarkerName)))
					{
						TickContext.MarkerTickContext.SetMarkerSyncStartPosition(EndPosition);
					}
				}

#if DO_CHECK
				//For debugging UE-54705
				FName InitialMarkerPrevious = TickContext.MarkerTickContext.GetMarkerSyncStartPosition().PreviousMarkerName;
				FName InitialMarkerEnd = TickContext.MarkerTickContext.GetMarkerSyncStartPosition().NextMarkerName;
				const bool bIsLeaderRecordValidPre = SyncGroup.ActivePlayers[0].MarkerTickRecord->IsValid(SyncGroup.ActivePlayers[0].bLooping);
				FMarkerTickRecord LeaderPreMarkerTickRecord = *SyncGroup.ActivePlayers[0].MarkerTickRecord;
#endif

				// Initialize with sync group leader being invalid first.
				ensureMsgf(SyncGroup.GroupLeaderIndex == INDEX_NONE, TEXT("SyncGroup %s had a non -1 group leader index of %d in asset %s"), *SyncGroupPair.Key.ToString(), SyncGroup.GroupLeaderIndex, *GetNameSafe(InProxy.GetAnimInstanceObject()));
				
				// Tick group leader, and try to find a new one in case the current one has an invalid position.
				int32 GroupLeaderIndex = 0;
				for (; GroupLeaderIndex < SyncGroup.ActivePlayers.Num(); ++GroupLeaderIndex)
				{
					FAnimTickRecord& GroupLeader = SyncGroup.ActivePlayers[GroupLeaderIndex];
					// if it has leader score
					SCOPE_CYCLE_COUNTER(STAT_TickAssetPlayerInstance);
					FScopeCycleCounterUObject Scope(GroupLeader.SourceAsset);

					// Enforce leader position and do not use previous group sync information as a starting point.
					if (GroupLeader.bOverridePositionWhenJoiningSyncGroupAsLeader)
					{
						if (PreviousGroup && PreviousGroup->ActivePlayers.IsValidIndex(PreviousGroup->GroupLeaderIndex))
						{
							const FAnimTickRecord & PrevLeader = PreviousGroup->ActivePlayers[PreviousGroup->GroupLeaderIndex];

							if (TickContext.CanUseMarkerPosition() && !PreviousGroup->ValidMarkers.IsEmpty())
							{
								const bool bIsAssetPlayerJustJoiningGroup = PrevLeader.MarkerTickRecord != GroupLeader.MarkerTickRecord || PrevLeader.DeltaTimeRecord != GroupLeader.DeltaTimeRecord;
								const bool bDidLeaderAssetChanged = PrevLeader.SourceAsset != GroupLeader.SourceAsset;
								
								if (bIsAssetPlayerJustJoiningGroup || bDidLeaderAssetChanged)
								{
									TickContext.MarkerTickContext.SetMarkerSyncStartPosition(FMarkerSyncAnimPosition(NAME_None, NAME_None, 0));	
								}
							}
							else
							{
								// Ensure no re-syncing occurs in order to overwrite previous leader position.
								TickContext.SetResyncToSyncGroup(false);
							}
						}
					}
					else
					{
						// Inertialization was requested therefore we resync to previous leader, if needed.
						if (GroupLeader.bRequestedInertialization)
						{
							// Ensure we have a previous group leader to sync to otherwise we play normally. 
							if (PreviousGroup && !PreviousGroup->ActivePlayers.IsEmpty())
							{
								// Only need to resync when using length based syncing since we initialized the tick context with the previous group's sync end position
								// and that will take care of marker based syncing.
								if (!TickContext.CanUseMarkerPosition() || !(PreviousGroup->ValidMarkers.Num() > 0))
								{
									const FAnimTickRecord & PreviousGroupLeader = PreviousGroup->ActivePlayers[PreviousGroup->GroupLeaderIndex];
									const bool bShouldResyncToSyncGroup = PreviousGroupLeader.LeaderScore >= GroupLeader.LeaderScore;
                
									// Sync to previous group leader if we have a lower score than them.
									TickContext.SetResyncToSyncGroup(bShouldResyncToSyncGroup);
								}
							}
						}
					}
					
					// Tick group leader's asset.
					TickContext.MarkerTickContext.MarkersPassedThisTick.Reset();
					TickContext.RootMotionMovementParams.Clear();
					GroupLeader.SourceAsset->TickAssetPlayer(GroupLeader, InProxy.NotifyQueue, TickContext);

					// Accumulate root motion if possible.
					if (RootMotionMode == ERootMotionMode::RootMotionFromEverything && TickContext.RootMotionMovementParams.bHasRootMotion)
					{
						AccumulateRootMotion(GroupLeader, TickContext);
					}

					// If we're not using marker based sync, we don't care, update and get out.
					if (TickContext.CanUseMarkerPosition() == false)
					{
						SyncGroup.PreviousAnimLengthRatio = TickContext.GetPreviousAnimationPositionRatio();
						SyncGroup.AnimLengthRatio = TickContext.GetAnimationPositionRatio();
						SyncGroup.GroupLeaderIndex = GroupLeaderIndex;
						break;
					}
					// Otherwise, the new position should contain the valid position for end, otherwise, we don't know where to sync to.
					else if (TickContext.MarkerTickContext.IsMarkerSyncEndValid())
					{
						// If this leader contains correct position, break
						SyncGroup.PreviousAnimLengthRatio = TickContext.GetPreviousAnimationPositionRatio();
						SyncGroup.AnimLengthRatio = TickContext.GetAnimationPositionRatio();
						SyncGroup.MarkerTickContext = TickContext.MarkerTickContext;
						SyncGroup.GroupLeaderIndex = GroupLeaderIndex;
						UE_LOGF(LogAnimMarkerSync, Log, "Previous Sync Group Marker Tick Context :\n%ls", *SyncGroup.MarkerTickContext.ToString());
						UE_LOGF(LogAnimMarkerSync, Log, "New Sync Group Marker Tick Context :\n%ls", *TickContext.MarkerTickContext.ToString());
						break;
					}
					// We have an invalid sync end position, keep searching for a valid leader.
					else
					{
						SyncGroup.PreviousAnimLengthRatio = TickContext.GetPreviousAnimationPositionRatio();
						SyncGroup.AnimLengthRatio = TickContext.GetAnimationPositionRatio();
						SyncGroup.GroupLeaderIndex = GroupLeaderIndex;
						UE_LOGF(LogAnimMarkerSync, Log, "Invalid position from Leader %d. Trying next leader", GroupLeaderIndex);
					}
				} 

				check(SyncGroup.GroupLeaderIndex != INDEX_NONE);
			
				// By this point we have found a valid group leader candidate.

				// Finalize sync group.
				// This is where the followers tick records are reset if the needed. 
				SyncGroup.Finalize(PreviousGroup);
				
				// Kick any records that requested being exclusive leaders but failed.
				for (int32 RecordIndex = GroupLeaderIndex + 1; RecordIndex < SyncGroup.ActivePlayers.Num(); ++RecordIndex)
				{
					// We do not modify the array since we can just skip ticking their asset player and just push that to run as ungrouped.
					if (SyncGroup.ActivePlayers[RecordIndex].bIsExclusiveLeader)
					{
						UngroupedActivePlayerArrays[GetSyncGroupWriteIndex()].Push(SyncGroup.ActivePlayers[RecordIndex]);
					}
				}
				
				// Ensure group leader's markers are valid, otherwise invalidate marker-based syncing for followers.
				if (TickContext.CanUseMarkerPosition())
				{
					const FMarkerSyncAnimPosition& MarkerStart = TickContext.MarkerTickContext.GetMarkerSyncStartPosition();
					FName SyncGroupName = SyncGroupPair.Key;
					FAnimTickRecord& GroupLeader = SyncGroup.ActivePlayers[SyncGroup.GroupLeaderIndex];
					FString LeaderAnimName = GroupLeader.SourceAsset->GetName();

					//  Updated logic in search for cause of UE-54705
					const bool bStartMarkerValid = (MarkerStart.PreviousMarkerName == NAME_None) || SyncGroup.ValidMarkers.Contains(MarkerStart.PreviousMarkerName);
					const bool bEndMarkerValid = (MarkerStart.NextMarkerName == NAME_None) || SyncGroup.ValidMarkers.Contains(MarkerStart.NextMarkerName);

					if (!bStartMarkerValid)
					{
#if DO_CHECK
						FString ErrorMsg = FString("Prev Marker name not valid for sync group.\n");
						ErrorMsg += FString::Format(TEXT("\tMarker {0} : SyncGroupName {1} : Leader {2}\n"), { MarkerStart.PreviousMarkerName.ToString(), SyncGroupName.ToString(), LeaderAnimName });
						ErrorMsg += FString::Format(TEXT("\tInitalPrev {0} : InitialNext {1} : GroupLeaderIndex {2}\n"), { InitialMarkerPrevious.ToString(), InitialMarkerEnd.ToString(), GroupLeaderIndex });
						ErrorMsg += FString::Format(TEXT("\tLeader (0 index) was originally valid: {0} | Record: {1}\n"), { bIsLeaderRecordValidPre, LeaderPreMarkerTickRecord.ToString() });
						ErrorMsg += FString::Format(TEXT("\t Valid Markers : {0}\n"), { SyncGroup.ValidMarkers.Num() });
						for (int32 MarkerIndex = 0; MarkerIndex < SyncGroup.ValidMarkers.Num(); ++MarkerIndex)
						{
							ErrorMsg += FString::Format(TEXT("\t\t{0}) '{1}'\n"), {MarkerIndex, SyncGroup.ValidMarkers[MarkerIndex].ToString()});
						}
						ensureMsgf(false, TEXT("%s"), *ErrorMsg);
#endif
						TickContext.InvalidateMarkerSync();
					}
					else if (!bEndMarkerValid)
					{
#if DO_CHECK
						FString ErrorMsg = FString("Next Marker name not valid for sync group.\n");
						ErrorMsg += FString::Format(TEXT("\tMarker {0} : SyncGroupName {1} : Leader {2}\n"), { MarkerStart.NextMarkerName.ToString(), SyncGroupName.ToString(), LeaderAnimName });
						ErrorMsg += FString::Format(TEXT("\tInitalPrev {0} : InitialNext {1} : GroupLeaderIndex {2}\n"), { InitialMarkerPrevious.ToString(), InitialMarkerEnd.ToString(), GroupLeaderIndex });
						ErrorMsg += FString::Format(TEXT("\tLeader (0 index) was originally valid: {0} | Record: {1}\n"), { bIsLeaderRecordValidPre, LeaderPreMarkerTickRecord.ToString() });
						ErrorMsg += FString::Format(TEXT("\t Valid Markers : {0}\n"), { SyncGroup.ValidMarkers.Num() });
						for (int32 MarkerIndex = 0; MarkerIndex < SyncGroup.ValidMarkers.Num(); ++MarkerIndex)
						{
							ErrorMsg += FString::Format(TEXT("\t\t{0}) '{1}'\n"), { MarkerIndex, SyncGroup.ValidMarkers[MarkerIndex].ToString() });
						}
						ensureMsgf(false, TEXT("%s"), *ErrorMsg);
#endif
						TickContext.InvalidateMarkerSync();
					}
				}

				// Update everything else to follow the leader, if there is more followers
				if (SyncGroup.ActivePlayers.Num() > GroupLeaderIndex + 1)
				{
					// if we don't have a good leader, no reason to convert to follower
                    // tick as leader
					TickContext.ConvertToFollower();

					for (int32 TickIndex = GroupLeaderIndex + 1; TickIndex < SyncGroup.ActivePlayers.Num(); ++TickIndex)
					{
						FAnimTickRecord& AssetPlayer = SyncGroup.ActivePlayers[TickIndex];

						// Skip since we only wanted to be leaders. (Asset player will be ticked as ungrouped)
						if (AssetPlayer.bIsExclusiveLeader)
						{
							continue;
						}
						
						// Tick follower's asset player.
						{
							SCOPE_CYCLE_COUNTER(STAT_TickAssetPlayerInstance);
							FScopeCycleCounterUObject Scope(AssetPlayer.SourceAsset);
							
							TickContext.SetResyncToSyncGroup(AssetPlayer.bRequestedInertialization);
							TickContext.RootMotionMovementParams.Clear();
							
							AssetPlayer.SourceAsset->TickAssetPlayer(AssetPlayer, InProxy.NotifyQueue, TickContext);
						}
					
						// Accumulate root motion if possible.
						if (RootMotionMode == ERootMotionMode::RootMotionFromEverything && TickContext.RootMotionMovementParams.bHasRootMotion)
						{
							AccumulateRootMotion(AssetPlayer, TickContext);
						}
					}
				}

#if ANIM_TRACE_ENABLED
				for(const FPassedMarker& PassedMarker : TickContext.MarkerTickContext.MarkersPassedThisTick)
				{
					TRACE_ANIM_SYNC_MARKER(CastChecked<UAnimInstance>(InProxy.GetAnimInstanceObject()), PassedMarker);
				}
#endif
			}
		}
	}

	// Tick ungrouped animation instances.
	{
		TArray<FAnimTickRecord>& UngroupedActivePlayers = UngroupedActivePlayerArrays[GetSyncGroupWriteIndex()];
		const TArray<FAnimTickRecord>& PreviousUngroupedActivePlayers = UngroupedActivePlayerArrays[GetSyncGroupReadIndex()];
		
		// Handle ungrouped animation asset players.
		for (int32 TickIndex = 0; TickIndex < UngroupedActivePlayers.Num(); ++TickIndex)
		{
			FAnimTickRecord& AssetPlayerToTick = UngroupedActivePlayers[TickIndex];

			// Get marker names.
			const TArray<FName>* UniqueNames = AssetPlayerToTick.SourceAsset->GetUniqueMarkerNames();
			const TArray<FName>& ValidMarkers = UniqueNames ? *UniqueNames : FMarkerTickContext::DefaultMarkerNames;

			// Single animation context used for ticking.
			const bool bOnlyOneAnimationInGroup = true;
			FAnimAssetTickContext TickContext(InDeltaSeconds, RootMotionMode, bOnlyOneAnimationInGroup, ValidMarkers);

			// Tick asset player.
			{
				SCOPE_CYCLE_COUNTER(STAT_TickAssetPlayerInstance);
				FScopeCycleCounterUObject Scope(AssetPlayerToTick.SourceAsset);
				AssetPlayerToTick.SourceAsset->TickAssetPlayer(AssetPlayerToTick, InProxy.NotifyQueue, TickContext);
			}

			// Accumulate root motion if possible.
			if (RootMotionMode == ERootMotionMode::RootMotionFromEverything && TickContext.RootMotionMovementParams.bHasRootMotion)
			{
				AccumulateRootMotion(AssetPlayerToTick, TickContext);
			}

#if ANIM_TRACE_ENABLED
			for(const FPassedMarker& PassedMarker : TickContext.MarkerTickContext.MarkersPassedThisTick)
			{
				TRACE_ANIM_SYNC_MARKER(CastChecked<UAnimInstance>(InProxy.GetAnimInstanceObject()), PassedMarker);
			}
#endif
		}
	}
	
	// Flip buffers now that we have ticked.
	TickSyncGroupWriteIndex();
}

bool FAnimSync::GetTimeToClosestMarker(FName SyncGroup, FName MarkerName, float& OutMarkerTime) const
{
	const FSyncGroupMap& SyncGroupMap = SyncGroupMaps[GetSyncGroupReadIndex()];

	if (const FAnimGroupInstance* SyncGroupInstancePtr = SyncGroupMap.Find(SyncGroup))
	{
		if (SyncGroupInstancePtr->bCanUseMarkerSync && SyncGroupInstancePtr->ActivePlayers.IsValidIndex(SyncGroupInstancePtr->GroupLeaderIndex))
		{
			const FMarkerSyncAnimPosition& EndPosition = SyncGroupInstancePtr->MarkerTickContext.GetMarkerSyncEndPosition();
			const FAnimTickRecord& Leader = SyncGroupInstancePtr->ActivePlayers[SyncGroupInstancePtr->GroupLeaderIndex];
			if (EndPosition.PreviousMarkerName == MarkerName)
			{
				OutMarkerTime = Leader.MarkerTickRecord->PreviousMarker.TimeToMarker;
				return true;
			}
			else if (EndPosition.NextMarkerName == MarkerName)
			{
				OutMarkerTime = Leader.MarkerTickRecord->NextMarker.TimeToMarker;
				return true;
			}
		}
	}
	return false;
}

bool FAnimSync::HasMarkerBeenHitThisFrame(FName SyncGroup, FName MarkerName) const
{
	const FSyncGroupMap& SyncGroupMap = SyncGroupMaps[GetSyncGroupReadIndex()];

	if (const FAnimGroupInstance* SyncGroupInstancePtr = SyncGroupMap.Find(SyncGroup))
	{
		if (SyncGroupInstancePtr->bCanUseMarkerSync)
		{
			return SyncGroupInstancePtr->MarkerTickContext.MarkersPassedThisTick.ContainsByPredicate([&MarkerName](const FPassedMarker& PassedMarker) -> bool
			{
				return PassedMarker.PassedMarkerName == MarkerName;
			});
		}
	}
	return false;
}

bool FAnimSync::IsSyncGroupBetweenMarkers(FName InSyncGroupName, FName PreviousMarker, FName NextMarker, bool bRespectMarkerOrder) const
{
	const FMarkerSyncAnimPosition& SyncGroupPosition = GetSyncGroupPosition(InSyncGroupName);
	if ((SyncGroupPosition.PreviousMarkerName == PreviousMarker) && (SyncGroupPosition.NextMarkerName == NextMarker))
	{
		return true;
	}

	if (!bRespectMarkerOrder)
	{
		return ((SyncGroupPosition.PreviousMarkerName == NextMarker) && (SyncGroupPosition.NextMarkerName == PreviousMarker));
	}

	return false;
}

FMarkerSyncAnimPosition FAnimSync::GetSyncGroupPosition(FName InSyncGroupName) const
{
	const FSyncGroupMap& SyncGroupMap = SyncGroupMaps[GetSyncGroupReadIndex()];

	if (const FAnimGroupInstance* SyncGroupInstancePtr = SyncGroupMap.Find(InSyncGroupName))
	{
		if (SyncGroupInstancePtr->bCanUseMarkerSync && SyncGroupInstancePtr->MarkerTickContext.IsMarkerSyncEndValid())
		{
			return SyncGroupInstancePtr->MarkerTickContext.GetMarkerSyncEndPosition();
		}
		else
		{
			return FMarkerSyncAnimPosition(NAME_None, NAME_None, SyncGroupInstancePtr->AnimLengthRatio);
		}
	}

	return FMarkerSyncAnimPosition();
}

bool FAnimSync::IsSyncGroupValid(FName InSyncGroupName) const
{
	const FSyncGroupMap& SyncGroupMap = SyncGroupMaps[GetSyncGroupReadIndex()];

	if (const FAnimGroupInstance* SyncGroupInstancePtr = SyncGroupMap.Find(InSyncGroupName))
	{
		// If we don't use Markers, we're always valid.
		return (!SyncGroupInstancePtr->bCanUseMarkerSync || SyncGroupInstancePtr->MarkerTickContext.IsMarkerSyncEndValid());
	}

	// If we're querying a sync group that doesn't exist, treat this as invalid
	return false;
}

FAnimTickRecord& FAnimSync::CreateUninitializedTickRecord(FAnimGroupInstance*& OutSyncGroupPtr, FName GroupName)
{
	// Find or create the sync group if there is one
	OutSyncGroupPtr = nullptr;
	if (GroupName != NAME_None)
	{
		FSyncGroupMap& SyncGroupMap = SyncGroupMaps[GetSyncGroupWriteIndex()];
		OutSyncGroupPtr = &SyncGroupMap.FindOrAdd(GroupName);
	}

	// Create the record
	FAnimTickRecord& TickRecord = ((OutSyncGroupPtr != nullptr) ? OutSyncGroupPtr->ActivePlayers : UngroupedActivePlayerArrays[GetSyncGroupWriteIndex()]).AddDefaulted_GetRef();
	return TickRecord;
}

FAnimTickRecord& FAnimSync::CreateUninitializedTickRecordInScope(FAnimInstanceProxy& InProxy, FAnimGroupInstance*& OutSyncGroupPtr, FName GroupName, EAnimSyncGroupScope Scope)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GroupName != NAME_None)
	{
		// If we have no main proxy or it is "this", force us to local
		if(InProxy.GetMainInstanceProxy() == nullptr || InProxy.GetMainInstanceProxy() == &InProxy)
		{
			Scope = EAnimSyncGroupScope::Local;
		}

		switch(Scope)
		{
		default:
			ensureMsgf(false, TEXT("FAnimSync::CreateUninitializedTickRecordInScope: Scope has invalid value %d"), EnumToUnderlyingType(Scope));
			// Fall through

		case EAnimSyncGroupScope::Local:
			return CreateUninitializedTickRecord(OutSyncGroupPtr, GroupName);
		case EAnimSyncGroupScope::Component:
			// Forward to the main instance to sync with animations there in TickAssetPlayerInstances()
			return InProxy.GetMainInstanceProxy()->CreateUninitializedTickRecord(OutSyncGroupPtr, GroupName);
		}
	}
	
	return CreateUninitializedTickRecord(OutSyncGroupPtr, GroupName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FAnimSync::AddReferencedObjects(UAnimInstance* InAnimInstance, FReferenceCollector& Collector)
{
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(UngroupedActivePlayerArrays); ++Index)
	{
		TArray<FAnimTickRecord>& UngroupedPlayers = UngroupedActivePlayerArrays[Index];
		for (FAnimTickRecord& TickRecord : UngroupedPlayers)
		{
			Collector.AddReferencedObject(TickRecord.SourceAsset, InAnimInstance);
		}

		FSyncGroupMap& SyncGroupMap = SyncGroupMaps[Index];
		for(TPair<FName, FAnimGroupInstance>& GroupPair : SyncGroupMap)
		{
			for (FAnimTickRecord& TickRecord : GroupPair.Value.ActivePlayers)
			{
				Collector.AddReferencedObject(TickRecord.SourceAsset, InAnimInstance);
			}
		}
	}
}

}}	// namespace UE::Anim
