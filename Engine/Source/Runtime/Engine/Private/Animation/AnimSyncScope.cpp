// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimSyncScope.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimTrace.h"

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::Anim::FAnimSyncGroupScope);

namespace UE { namespace Anim {

FAnimSyncDebugInfo::FAnimSyncDebugInfo(const FAnimationBaseContext& InContext)
	: SourceProxy(InContext.AnimInstanceProxy)
	, SourceNodeId(InContext.GetCurrentNodeId())
{}

FAnimSyncGroupScope::FAnimSyncGroupScope(const FAnimationBaseContext& InContext, FName InSyncGroup, EAnimGroupRole::Type InGroupRole)
	: NodeId(InContext.GetCurrentNodeId())
	, Proxy(*InContext.AnimInstanceProxy)
	, OuterProxy(nullptr)
	, SyncGroup(InSyncGroup)
	, GroupRole(InGroupRole)
{
	// If we have an outer message, grab its proxy for forwarding
	if(const FAnimSyncGroupScope* OuterMessage = InContext.GetMessage<FAnimSyncGroupScope>())
	{
		OuterProxy = OuterMessage->OuterProxy != nullptr ? OuterMessage->OuterProxy : &OuterMessage->Proxy;
	}
}

void FAnimSyncGroupScope::AddTickRecord(const FAnimTickRecord& InTickRecord, const FAnimSyncParams& InSyncParams, const FAnimSyncDebugInfo& InDebugInfo)
{
	FAnimSyncParams NewSyncParams = InSyncParams;

	// Apply method to transform params if necessary
	switch(InSyncParams.Method)
	{
	default:
	case EAnimSyncMethod::DoNotSync:
		check(InSyncParams.GroupName == NAME_None);
		break;
	case EAnimSyncMethod::SyncGroup:
		check(InSyncParams.GroupName != NAME_None);
		break;
	case EAnimSyncMethod::Graph:
		// Override sync group/role supplied with our group
		check(InSyncParams.GroupName == NAME_None);
		NewSyncParams.GroupName = SyncGroup;
		NewSyncParams.Role = GroupRole;
#if !UE_BUILD_SHIPPING
		if(InDebugInfo.SourceProxy != nullptr && InDebugInfo.SourceNodeId != INDEX_NONE)
		{
#if WITH_EDITORONLY_DATA
			Proxy.RecordNodeAttribute(*InDebugInfo.SourceProxy, NodeId, InDebugInfo.SourceNodeId, FAnimSync::Attribute);
			InDebugInfo.SourceProxy->RecordNodeSync(InDebugInfo.SourceNodeId, SyncGroup);
#endif
			TRACE_ANIM_NODE_ATTRIBUTE(Proxy, *InDebugInfo.SourceProxy, NodeId, InDebugInfo.SourceNodeId, FAnimSync::Attribute);
			TRACE_ANIM_NODE_SYNC(*InDebugInfo.SourceProxy, InDebugInfo.SourceNodeId, SyncGroup);
		}
#endif
		break;
	}

	// Forward to outer instance if we have one
	if(OuterProxy)
	{
		OuterProxy->AddTickRecord(InTickRecord, NewSyncParams);
	}
	else
	{
		Proxy.AddTickRecord(InTickRecord, NewSyncParams);
	}
}

void FAnimSyncGroupScope::SetMirror(const UMirrorDataTable* MirrorDataTable)
{
	// Forward to outer instance if we have one
	if (OuterProxy)
	{
		OuterProxy->SetSyncMirror(MirrorDataTable);
	}
	else
	{
		Proxy.SetSyncMirror(MirrorDataTable);
	}
}



}}	// namespace UE::Anim
