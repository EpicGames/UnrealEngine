// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimTrace.h"

#if ANIM_TRACE_ENABLED

#include "ObjectTrace.h"
#include "TraceFilter.h"
#include "Animation/AnimInstanceProxy.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/ExternalMorphSet.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Trace/Trace.inl"
#include "UObject/UObjectAnnotation.h"
#include "Engine/PoseWatch.h"

UE_TRACE_CHANNEL_DEFINE(AnimationChannel, "Animation tick records, skeletal mesh poses, and animation graph evaluation.");

UE_TRACE_EVENT_BEGIN(Animation, TickRecord3)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, AssetId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(float, BlendWeight)
	UE_TRACE_EVENT_FIELD(float, PlaybackTime)
	UE_TRACE_EVENT_FIELD(float, RootMotionWeight)
	UE_TRACE_EVENT_FIELD(float, PlayRate)
	UE_TRACE_EVENT_FIELD(float, BlendSpacePositionX)
	UE_TRACE_EVENT_FIELD(float, BlendSpacePositionY)
	UE_TRACE_EVENT_FIELD(float, BlendSpaceFilteredPositionX)
	UE_TRACE_EVENT_FIELD(float, BlendSpaceFilteredPositionY)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(bool, Looping)
	UE_TRACE_EVENT_FIELD(bool, IsBlendSpace)
	UE_TRACE_EVENT_FIELD(uint32, SyncGroupNameId)
	UE_TRACE_EVENT_FIELD(bool, IsSyncGroupLeader)
	UE_TRACE_EVENT_FIELD(float, SyncLeaderScore)
	UE_TRACE_EVENT_FIELD(float, TimeToPrevMarker)
	UE_TRACE_EVENT_FIELD(float, TimeToNextMarker)
	UE_TRACE_EVENT_FIELD(int32, PrevMarkerIndex)
	UE_TRACE_EVENT_FIELD(int32, NextMarkerIndex)

UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, SkeletalMesh3, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint64, Id)
	UE_TRACE_EVENT_FIELD(uint64, SkeletonId)
	UE_TRACE_EVENT_FIELD(int32[], ParentIndices)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, SkeletalMeshComponent3)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, ComponentId)
	UE_TRACE_EVENT_FIELD(uint64, MeshId)
	UE_TRACE_EVENT_FIELD(float[], ComponentToWorld)
	UE_TRACE_EVENT_FIELD(float[], Pose)
	UE_TRACE_EVENT_FIELD(uint32[], CurveIds)
	UE_TRACE_EVENT_FIELD(float[], CurveValues)
	UE_TRACE_EVENT_FIELD(uint16, LodIndex)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, SkeletalMeshComponent4)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, ComponentId)
	UE_TRACE_EVENT_FIELD(uint64, MeshId)
	UE_TRACE_EVENT_FIELD(bool, IsVisible)
	UE_TRACE_EVENT_FIELD(float[], ComponentToWorld)
	UE_TRACE_EVENT_FIELD(float[], Pose)
	UE_TRACE_EVENT_FIELD(uint32[], CurveIds)
	UE_TRACE_EVENT_FIELD(float[], CurveValues)
	UE_TRACE_EVENT_FIELD(uint16, LodIndex)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(float[], ExternalMorphSetWeights)		// Concatenated array [WeightsOfSet0, WeightsOfSet1, ..., WeightsOfSetN].
	UE_TRACE_EVENT_FIELD(int32[], ExternalMorphSetWeightCounts)	// The number of weights in each external morph set.
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, SkeletalMeshFrame)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, ComponentId)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimGraph)
	UE_TRACE_EVENT_FIELD(uint64, StartCycle)
	UE_TRACE_EVENT_FIELD(uint64, EndCycle)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeCount)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(uint8, Phase)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeStart)
	UE_TRACE_EVENT_FIELD(uint64, StartCycle)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, PreviousNodeId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(float, Weight)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(uint8, Phase)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)	
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, DisplayName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeEnd)
	UE_TRACE_EVENT_FIELD(uint64, EndCycle)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeAttribute)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, SourceAnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, TargetAnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, SourceNodeId)
	UE_TRACE_EVENT_FIELD(int32, TargetNodeId)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueBool)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(bool, Value)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueInt)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(int32, Value)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueFloat)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(float, Value)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueVector2D)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(float, ValueX)
	UE_TRACE_EVENT_FIELD(float, ValueY)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueVector)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(float, ValueX)
	UE_TRACE_EVENT_FIELD(float, ValueY)
	UE_TRACE_EVENT_FIELD(float, ValueZ)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueString)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Value)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueObject)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, Value)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueClass)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, Value)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimNodeValueAnimNode)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, Value)
	UE_TRACE_EVENT_FIELD(uint64, ValueAnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, AnimSequencePlayer)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(float, Position)
	UE_TRACE_EVENT_FIELD(float, Length)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, StateMachineState)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(int32, StateMachineIndex)
	UE_TRACE_EVENT_FIELD(int32, StateIndex)
	UE_TRACE_EVENT_FIELD(float, StateWeight)
	UE_TRACE_EVENT_FIELD(float, ElapsedTime)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, Name, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, Notify2)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, AssetId)
	UE_TRACE_EVENT_FIELD(uint64, NotifyId)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(float, Time)
	UE_TRACE_EVENT_FIELD(float, Duration)
	UE_TRACE_EVENT_FIELD(uint8, NotifyEventType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, SyncMarker2)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, Montage2)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, MontageId)
	UE_TRACE_EVENT_FIELD(uint32, CurrentSectionNameId)
	UE_TRACE_EVENT_FIELD(uint32, NextSectionNameId)
	UE_TRACE_EVENT_FIELD(float, Weight)
	UE_TRACE_EVENT_FIELD(float, DesiredWeight)
	UE_TRACE_EVENT_FIELD(float, Position)
	UE_TRACE_EVENT_FIELD(uint16, FrameCounter)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, Sync)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(int32, SourceNodeId)
	UE_TRACE_EVENT_FIELD(uint32, GroupNameId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, PoseWatch2)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, ComponentId)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, PoseWatchId)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(uint32, Color)
	UE_TRACE_EVENT_FIELD(float[], WorldTransform)
	UE_TRACE_EVENT_FIELD(float[], BoneTransforms)
	UE_TRACE_EVENT_FIELD(float[], CurveValues)
	UE_TRACE_EVENT_FIELD(uint32[], CurveIds)
	UE_TRACE_EVENT_FIELD(uint16[], RequiredBones)
	UE_TRACE_EVENT_FIELD(bool, bIsEnabled)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Animation, Inertialization)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, ComponentId)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint32, NodeId)
	UE_TRACE_EVENT_FIELD(float, Weight)
	UE_TRACE_EVENT_FIELD(uint8, Type)
UE_TRACE_EVENT_END()

FAutoConsoleVariable CVarRecordExternalMorphTargets(
	TEXT("a.RecordExternalMorphTargets"),
	false,
	TEXT("Record the external morph target weights inside animation insights. On default this is disabled, because it can slow down recording."),
	ECVF_Default
);

// Object annotations used for tracing
FUObjectAnnotationSparseBool GSkeletalMeshTraceAnnotations;

// Map used for unique name output
TMap<FName, uint32> GAnimTraceNames;

// Global unique name index
uint32 GAnimTraceCurrentNameId = 1;

// Critical section used to lock global name map & index
FCriticalSection GAnimTraceNameCriticalSection;

void FAnimTrace::Reset()
{
	GSkeletalMeshTraceAnnotations.ClearAll();
	GAnimTraceNames.Reset();
	GAnimTraceCurrentNameId = 1;
}

// Scratch buffers for various traces to avoid allocation churn.
// These can be removed when lambda support is added for array fields to remove a memcpy.
struct FAnimTraceScratchBuffers : public TThreadSingleton<FAnimTraceScratchBuffers>
{
	// Curve values/IDs for skeletal mesh component
	TArray<float> CurveValues;
	TArray<uint32> CurveIds;
	TArray<float> ExternalMorphTargetWeights;
	TArray<int32> ExternalMorphTargetWeightCounts;

	// Parent indices for skeletal meshes
	TArray<int32> ParentIndices;
};

class FSuspendCounter : public TThreadSingleton<FSuspendCounter>
{
public:
	FSuspendCounter()
		: SuspendCount(0)
	{}

	int32 SuspendCount;
};

FAnimTrace::FScopedAnimNodeTraceSuspend::FScopedAnimNodeTraceSuspend()
{
	FSuspendCounter::Get().SuspendCount++;
}

FAnimTrace::FScopedAnimNodeTraceSuspend::~FScopedAnimNodeTraceSuspend()
{
	FSuspendCounter::Get().SuspendCount--;
	check(FSuspendCounter::Get().SuspendCount >= 0);
}

FAnimTrace::FScopedAnimNodeTrace::FScopedAnimNodeTrace(const FAnimationInitializeContext& InContext)
	: Context(InContext)
{
	if(FSuspendCounter::Get().SuspendCount == 0)
	{
		OutputAnimNodeStart(InContext, FPlatformTime::Cycles64(), InContext.GetPreviousNodeId(), InContext.GetCurrentNodeId(), 0.0f, 0.0f, (__underlying_type(EPhase))EPhase::Initialize);
	}
}

FAnimTrace::FScopedAnimNodeTrace::FScopedAnimNodeTrace(const FAnimationUpdateContext& InContext)
	: Context(InContext)
{
	if(FSuspendCounter::Get().SuspendCount == 0)
	{
		OutputAnimNodeStart(InContext, FPlatformTime::Cycles64(), InContext.GetPreviousNodeId(), InContext.GetCurrentNodeId(), InContext.GetFinalBlendWeight(), InContext.GetRootMotionWeightModifier(), (__underlying_type(EPhase))EPhase::Update);
	}
}

FAnimTrace::FScopedAnimNodeTrace::FScopedAnimNodeTrace(const FAnimationCacheBonesContext& InContext)
	: Context(InContext)
{
	if(FSuspendCounter::Get().SuspendCount == 0)
	{
		OutputAnimNodeStart(InContext, FPlatformTime::Cycles64(), InContext.GetPreviousNodeId(), InContext.GetCurrentNodeId(), 0.0f, 0.0f, (__underlying_type(EPhase))EPhase::CacheBones);
	}
}

FAnimTrace::FScopedAnimNodeTrace::FScopedAnimNodeTrace(const FPoseContext& InContext)
	: Context(InContext)
{
	if(FSuspendCounter::Get().SuspendCount == 0)
	{
		OutputAnimNodeStart(InContext, FPlatformTime::Cycles64(), InContext.GetPreviousNodeId(), InContext.GetCurrentNodeId(), 0.0f, 0.0f, (__underlying_type(EPhase))EPhase::Evaluate);
	}
}

FAnimTrace::FScopedAnimNodeTrace::FScopedAnimNodeTrace(const FComponentSpacePoseContext& InContext)
	: Context(InContext)
{
	if(FSuspendCounter::Get().SuspendCount == 0)
	{
		OutputAnimNodeStart(InContext, FPlatformTime::Cycles64(), InContext.GetPreviousNodeId(), InContext.GetCurrentNodeId(), 0.0f, 0.0f, (__underlying_type(EPhase))EPhase::Evaluate);
	}
}

FAnimTrace::FScopedAnimNodeTrace::~FScopedAnimNodeTrace()
{
	if(FSuspendCounter::Get().SuspendCount == 0)
	{
		OutputAnimNodeEnd(Context, FPlatformTime::Cycles64());
	}
}

FAnimTrace::FScopedAnimGraphTrace::FScopedAnimGraphTrace(const FAnimationInitializeContext& InContext)
	: StartCycle(FPlatformTime::Cycles64())
	, Context(InContext)
	, Phase(EPhase::Initialize)
{}

FAnimTrace::FScopedAnimGraphTrace::FScopedAnimGraphTrace(const FAnimationUpdateContext& InContext)
	: StartCycle(FPlatformTime::Cycles64())
	, Context(InContext)
	, Phase(EPhase::Update)
{}

FAnimTrace::FScopedAnimGraphTrace::FScopedAnimGraphTrace(const FAnimationCacheBonesContext& InContext)
	: StartCycle(FPlatformTime::Cycles64())
	, Context(InContext)
	, Phase(EPhase::CacheBones)
{}

FAnimTrace::FScopedAnimGraphTrace::FScopedAnimGraphTrace(const FPoseContext& InContext)
	: StartCycle(FPlatformTime::Cycles64())
	, Context(InContext)
	, Phase(EPhase::Evaluate)
{}

FAnimTrace::FScopedAnimGraphTrace::FScopedAnimGraphTrace(const FComponentSpacePoseContext& InContext)
	: StartCycle(FPlatformTime::Cycles64())
	, Context(InContext)
	, Phase(EPhase::Evaluate)
{}

FAnimTrace::FScopedAnimGraphTrace::~FScopedAnimGraphTrace()
{
	OutputAnimGraph(Context, StartCycle, FPlatformTime::Cycles64(), (__underlying_type(EPhase))Phase);
}

void FAnimTrace::OutputAnimTickRecord(const FAnimationBaseContext& InContext, const FAnimTickRecord& InTickRecord)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	if (InTickRecord.SourceAsset)
	{
		// Use the source proxy for attribution if the tick record originated from a different proxy (e.g. a linked anim graph), 
		// while keeping the context proxy for sync group lookup since tick records always live in the main proxy's sync regardless of which linked graph created them.
		FAnimInstanceProxy* AttributionProxy = (InTickRecord.SourceProxy && InTickRecord.SourceProxy != InContext.AnimInstanceProxy) ? InTickRecord.SourceProxy : InContext.AnimInstanceProxy;
		UObject* AnimInstance = AttributionProxy->GetAnimInstanceObject();
		TRACE_OBJECT(AnimInstance);

		TRACE_OBJECT(InTickRecord.SourceAsset);

		float PlaybackTime = 0.0f;
		if (InTickRecord.TimeAccumulator)
		{
			PlaybackTime = *InTickRecord.TimeAccumulator;
		}
		
		if(InTickRecord.SourceAsset->IsA<UAnimMontage>())
		{
			PlaybackTime = InTickRecord.Montage.CurrentPosition;
		}

		float BlendSpacePositionX = 0.0f;
		float BlendSpacePositionY = 0.0f;
		float BlendSpaceFilteredPositionX = 0.0f;
		float BlendSpaceFilteredPositionY = 0.0f;
		const bool bIsBlendSpace = InTickRecord.SourceAsset->IsA<UBlendSpace>();
		if(bIsBlendSpace)
		{
			BlendSpacePositionX = InTickRecord.BlendSpace.BlendSpacePositionX;
			BlendSpacePositionY = InTickRecord.BlendSpace.BlendSpacePositionY;
			BlendSpaceFilteredPositionX = InTickRecord.BlendSpace.BlendFilter->GetFilterLastOutput().X;
			BlendSpaceFilteredPositionY = InTickRecord.BlendSpace.BlendFilter->GetFilterLastOutput().Y;
		}

		// Figure out which sync group is this tick record located, if any at all.
		const TMap<FName, FAnimGroupInstance>& GroupedTickRecords = InContext.AnimInstanceProxy->GetSyncGroupMapRead();
		
		static FName NoSyncGroupName(EName::None);
		FName SyncGroupName = NoSyncGroupName;
		bool IsSyncGroupLeader = false;
		
		for (const TTuple<FName, FAnimGroupInstance>& TickRecordGroup : GroupedTickRecords)
		{
			int32 GroupIdx = 0;
			for (const FAnimTickRecord& TickRecord : TickRecordGroup.Value.ActivePlayers)
			{
				if (&TickRecord == &InTickRecord)
				{
					SyncGroupName = TickRecordGroup.Key;
					IsSyncGroupLeader = GroupIdx == TickRecordGroup.Value.GroupLeaderIndex;
				}
				++GroupIdx;
			}
		}

		uint32 SyncGroupNameId = FAnimTrace::OutputName(SyncGroupName);
		
		UE_TRACE_LOG(Animation, TickRecord3, AnimationChannel)
			<< TickRecord3.Cycle(FPlatformTime::Cycles64())
			<< TickRecord3.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
			<< TickRecord3.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
			<< TickRecord3.AssetId(FObjectTrace::GetObjectId(InTickRecord.SourceAsset))
			<< TickRecord3.NodeId(InContext.GetCurrentNodeId())
			<< TickRecord3.BlendWeight(InTickRecord.EffectiveBlendWeight)
			<< TickRecord3.PlaybackTime(PlaybackTime)
			<< TickRecord3.RootMotionWeight(InTickRecord.RootMotionWeightModifier)
			<< TickRecord3.PlayRate(InTickRecord.PlayRateMultiplier)
			<< TickRecord3.BlendSpacePositionX(BlendSpacePositionX)
			<< TickRecord3.BlendSpacePositionY(BlendSpacePositionY)
			<< TickRecord3.BlendSpaceFilteredPositionX(BlendSpaceFilteredPositionX)
			<< TickRecord3.BlendSpaceFilteredPositionY(BlendSpaceFilteredPositionY)
			<< TickRecord3.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
			<< TickRecord3.Looping(InTickRecord.bLooping)
			<< TickRecord3.IsBlendSpace(bIsBlendSpace)
			<< TickRecord3.SyncGroupNameId(SyncGroupNameId)
			<< TickRecord3.IsSyncGroupLeader(IsSyncGroupLeader)
			<< TickRecord3.SyncLeaderScore(InTickRecord.LeaderScore)
			<< TickRecord3.PrevMarkerIndex(InTickRecord.MarkerTickRecord ? InTickRecord.MarkerTickRecord->PreviousMarker.MarkerIndex : -1)
			<< TickRecord3.NextMarkerIndex(InTickRecord.MarkerTickRecord ? InTickRecord.MarkerTickRecord->NextMarker.MarkerIndex : -1)
			<< TickRecord3.TimeToPrevMarker(InTickRecord.MarkerTickRecord ? InTickRecord.MarkerTickRecord->PreviousMarker.TimeToMarker : 0)
			<< TickRecord3.TimeToNextMarker(InTickRecord.MarkerTickRecord ? InTickRecord.MarkerTickRecord->NextMarker.TimeToMarker : 0);
	}
}

void FAnimTrace::OutputSkeletalMesh(const USkeletalMesh* InMesh)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled || InMesh == nullptr)
	{
		return;
	}

	if(GSkeletalMeshTraceAnnotations.Get(InMesh))
	{
		return;
	}

	TRACE_OBJECT(InMesh);
	TRACE_OBJECT(InMesh->GetSkeleton());

	uint32 BoneCount = (uint32)InMesh->GetRefSkeleton().GetNum();

	TArray<int32>& ParentIndices = FAnimTraceScratchBuffers::Get().ParentIndices;
	ParentIndices.Reset();
	ParentIndices.SetNumUninitialized(BoneCount);

	int32 BoneIndex = 0;
	for(const FMeshBoneInfo& BoneInfo : InMesh->GetRefSkeleton().GetRefBoneInfo())
	{
		ParentIndices[BoneIndex++] = BoneInfo.ParentIndex;
	}

	UE_TRACE_LOG(Animation, SkeletalMesh3, AnimationChannel, ParentIndices.Num() * sizeof(int32))
		<< SkeletalMesh3.Id(FObjectTrace::GetObjectId(InMesh))
		<< SkeletalMesh3.SkeletonId(FObjectTrace::GetObjectId(InMesh->GetSkeleton()))
		<< SkeletalMesh3.ParentIndices(ParentIndices.GetData(), ParentIndices.Num());

	GSkeletalMeshTraceAnnotations.Set(InMesh);
}

uint32 FAnimTrace::OutputName(const FName& InName)
{
	uint32 NameId = 0;
	bool bShouldTrace = false;
	if(InName != NAME_None)
	{
		FScopeLock ScopeLock(&GAnimTraceNameCriticalSection);

		uint32* ExistingIdPtr = GAnimTraceNames.Find(InName);
		if(ExistingIdPtr == nullptr)
		{
			NameId = GAnimTraceCurrentNameId++;
			GAnimTraceNames.Add(InName, NameId);
			bShouldTrace = true;
		}
		else
		{
			NameId = *ExistingIdPtr;
		}
	}

	if(bShouldTrace)
	{
		TCHAR Buffer[FName::StringBufferSize];
		int32 NameStringLength = InName.ToString(Buffer);

		UE_TRACE_LOG(Animation, Name, AnimationChannel, NameStringLength * sizeof(TCHAR))
			<< Name.Id(NameId)
			<< Name.Name(Buffer, NameStringLength);
	}

	return NameId;
}

void FAnimTrace::OutputSkeletalMeshComponent(const USkeletalMeshComponent* InComponent)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled || InComponent == nullptr)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(InComponent))
	{
		return;
	}

	int32 BoneCount = InComponent->GetComponentSpaceTransforms().Num();
	int32 CurveCount = 0;
	UAnimInstance* AnimInstance = InComponent->GetAnimInstance();

	if(AnimInstance)
	{
		for(EAnimCurveType CurveType : TEnumRange<EAnimCurveType>())
		{
			CurveCount += AnimInstance->GetAnimationCurveList(CurveType).Num();
		}
	}

	// Get the external morph target sets.
	const int32 LOD = InComponent->GetPredictedLODLevel();
	
	FExternalMorphSets* ExternalMorphSet = nullptr;
	if (InComponent->IsValidExternalMorphSetLODIndex(LOD) && CVarRecordExternalMorphTargets->GetBool())
	{
		ExternalMorphSet = const_cast<FExternalMorphSets*>(&InComponent->GetExternalMorphSets(LOD));
	}

	if (BoneCount > 0 || CurveCount > 0 || (ExternalMorphSet && !ExternalMorphSet->IsEmpty()))
	{
		TRACE_OBJECT(InComponent);
		TRACE_SKELETAL_MESH(InComponent->GetSkeletalMeshAsset());

		TArray<float>& CurveValues = FAnimTraceScratchBuffers::Get().CurveValues;
		CurveValues.Reset();
		CurveValues.SetNumUninitialized(CurveCount);
		TArray<uint32>& CurveIds = FAnimTraceScratchBuffers::Get().CurveIds;
		CurveIds.Reset();
		CurveIds.SetNumUninitialized(CurveCount);

		if(CurveCount > 0 && AnimInstance)
		{
			int32 CurveIndex = 0;
			for(EAnimCurveType CurveType : TEnumRange<EAnimCurveType>())
			{
				for(TPair<FName, float> CurvePair : AnimInstance->GetAnimationCurveList(CurveType))
				{
					CurveIds[CurveIndex] = OutputName(CurvePair.Key);
					CurveValues[CurveIndex] = CurvePair.Value;
					CurveIndex++;
				}
			}
		}

		TArray<float>& ExternalMorphTraceWeights = FAnimTraceScratchBuffers::Get().ExternalMorphTargetWeights;
		TArray<int32>& ExternalMorphTraceWeightCounts = FAnimTraceScratchBuffers::Get().ExternalMorphTargetWeightCounts;
		ExternalMorphTraceWeights.Reset();
		ExternalMorphTraceWeightCounts.Reset();

		// Get the weights for all external morph sets.
		if (ExternalMorphSet && !ExternalMorphSet->IsEmpty())
		{
			const FExternalMorphWeightData& ExternalMorphWeightData = InComponent->GetExternalMorphWeights(LOD);
			const int32 NumMorphWeightSets = ExternalMorphWeightData.MorphSets.Num();
			const int32 NumMorphDataSets = ExternalMorphSet->Num();
			if (!ExternalMorphWeightData.MorphSets.IsEmpty() && NumMorphDataSets == NumMorphWeightSets)
			{
				// Allocate enough data in the scratch buffer.
				ExternalMorphTraceWeightCounts.SetNumZeroed(NumMorphDataSets);

				int32 SetIndex = 0;
				for (const auto& CurrentSet : ExternalMorphWeightData.MorphSets)
				{
					ExternalMorphTraceWeightCounts[SetIndex] = CurrentSet.Value.Weights.Num();
					ExternalMorphTraceWeights.Append(CurrentSet.Value.Weights);
					SetIndex++;
				}
			}
		}

		UE_TRACE_LOG(Animation, SkeletalMeshComponent4, AnimationChannel)
			<< SkeletalMeshComponent4.Cycle(FPlatformTime::Cycles64())
			<< SkeletalMeshComponent4.RecordingTime(FObjectTrace::GetWorldElapsedTime(InComponent->GetWorld()))
			<< SkeletalMeshComponent4.ComponentId(FObjectTrace::GetObjectId(InComponent))
			<< SkeletalMeshComponent4.MeshId(FObjectTrace::GetObjectId(InComponent->GetSkeletalMeshAsset()))
			<< SkeletalMeshComponent4.IsVisible(InComponent->GetVisibleFlag())
			<< SkeletalMeshComponent4.ComponentToWorld(reinterpret_cast<const float*>(&InComponent->GetComponentToWorld()), sizeof(FTransform) / sizeof(float))
			<< SkeletalMeshComponent4.Pose(reinterpret_cast<const float*>(InComponent->GetComponentSpaceTransforms().GetData()), BoneCount * (sizeof(FTransform) / sizeof(float)))
			<< SkeletalMeshComponent4.CurveIds(CurveIds.GetData(), CurveIds.Num())
			<< SkeletalMeshComponent4.CurveValues(CurveValues.GetData(), CurveValues.Num())
			<< SkeletalMeshComponent4.LodIndex((uint16)InComponent->GetPredictedLODLevel())
			<< SkeletalMeshComponent4.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(InComponent))
			<< SkeletalMeshComponent4.ExternalMorphSetWeights(ExternalMorphTraceWeights.GetData(), ExternalMorphTraceWeights.Num())
			<< SkeletalMeshComponent4.ExternalMorphSetWeightCounts(ExternalMorphTraceWeightCounts.GetData(), ExternalMorphTraceWeightCounts.Num());
	}
}

void FAnimTrace::OutputSkeletalMeshFrame(const USkeletalMeshComponent* InComponent)
{
	if (CAN_TRACE_OBJECT(InComponent))
	{
		TRACE_OBJECT(InComponent);
		UE_TRACE_LOG(Animation, SkeletalMeshFrame, AnimationChannel)
			<< SkeletalMeshFrame.Cycle(FPlatformTime::Cycles64())
			<< SkeletalMeshFrame.ComponentId(FObjectTrace::GetObjectId(InComponent))
			<< SkeletalMeshFrame.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(InComponent));
	}
}

void FAnimTrace::OutputAnimGraph(const FAnimationBaseContext& InContext, uint64 InStartCycle, uint64 InEndCycle, uint8 InPhase)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);
	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	const UAnimInstance* AnimInstance = Cast<UAnimInstance>(InContext.AnimInstanceProxy->GetAnimInstanceObject());
	const UAnimBlueprintGeneratedClass* BPClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass());

	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimGraph, AnimationChannel)
		<< AnimGraph.StartCycle(InStartCycle)
		<< AnimGraph.EndCycle(InEndCycle)
		<< AnimGraph.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimGraph.NodeCount(BPClass ? BPClass->GetAnimNodeProperties().Num() : 0)
		<< AnimGraph.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimGraph.Phase(InPhase);
}

void FAnimTrace::OutputAnimNodeStart(const FAnimationBaseContext& InContext, uint64 InStartCycle, int32 InPreviousNodeId, int32 InNodeId, float InBlendWeight, float InRootMotionWeight, uint8 InPhase)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	if(InNodeId == INDEX_NONE)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	FString NameString, DisplayNameString;
	IAnimClassInterface* AnimBlueprintClass = InContext.GetAnimClass();
	if(AnimBlueprintClass)
	{
		const TArray<FStructProperty*>& AnimNodeProperties = AnimBlueprintClass->GetAnimNodeProperties();
		check(AnimNodeProperties.IsValidIndex(InNodeId));
		FStructProperty* LinkedProperty = AnimNodeProperties[InNodeId];
		check(LinkedProperty->Struct);

		NameString = LinkedProperty->Struct->GetFName().ToString();

#if WITH_EDITOR
		DisplayNameString = LinkedProperty->Struct->GetDisplayNameText().ToString();
#else
		DisplayNameString = LinkedProperty->Struct->GetName();
#endif
		DisplayNameString.RemoveFromStart(TEXT("Anim Node "));
	}
	else
	{
		DisplayNameString = TEXT("Anim Node");
	}

	check(InPreviousNodeId != InNodeId);

	UE_TRACE_LOG(Animation, AnimNodeStart, AnimationChannel)
		<< AnimNodeStart.StartCycle(InStartCycle)
		<< AnimNodeStart.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeStart.PreviousNodeId(InPreviousNodeId)
		<< AnimNodeStart.NodeId(InNodeId)
		<< AnimNodeStart.Weight(InBlendWeight)
		<< AnimNodeStart.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeStart.Phase(InPhase)
		<< AnimNodeStart.Name(*NameString, NameString.Len())	
		<< AnimNodeStart.DisplayName(*DisplayNameString, DisplayNameString.Len());
}

void FAnimTrace::OutputAnimNodeEnd(const FAnimationBaseContext& InContext, uint64 InEndCycle)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimNodeEnd, AnimationChannel)
		<< AnimNodeEnd.EndCycle(InEndCycle)
		<< AnimNodeEnd.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance));
}

void FAnimTrace::OutputAnimNodeAttribute(const FAnimInstanceProxy& InTargetProxy, const FAnimInstanceProxy& InSourceProxy, int32 InTargetNodeId, int32 InSourceNodeId, FName InAttribute)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(InSourceProxy.GetSkelMeshComponent()) || CANNOT_TRACE_OBJECT(InTargetProxy.GetSkelMeshComponent()))
	{
		return;
	}

	const UObject* SourceAnimInstance = InSourceProxy.GetAnimInstanceObject();
	TRACE_OBJECT(SourceAnimInstance)
	const UObject* TargetAnimInstance = InTargetProxy.GetAnimInstanceObject();
	TRACE_OBJECT(TargetAnimInstance);

	uint32 NameId = OutputName(InAttribute);

	UE_TRACE_LOG(Animation, AnimNodeAttribute, AnimationChannel)
		<< AnimNodeAttribute.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeAttribute.SourceAnimInstanceId(FObjectTrace::GetObjectId(SourceAnimInstance))
		<< AnimNodeAttribute.TargetAnimInstanceId(FObjectTrace::GetObjectId(TargetAnimInstance))
		<< AnimNodeAttribute.SourceNodeId(InSourceNodeId)
		<< AnimNodeAttribute.TargetNodeId(InTargetNodeId)
		<< AnimNodeAttribute.NameId(NameId);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, bool InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimNodeValueBool, AnimationChannel)
		<< AnimNodeValueBool.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueBool.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueBool.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueBool.NodeId(NodeIndex)
		<< AnimNodeValueBool.Value(InValue)
		<< AnimNodeValueBool.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueBool.Key(InKey);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, int32 InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimNodeValueInt, AnimationChannel)
		<< AnimNodeValueInt.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueInt.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueInt.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueInt.NodeId(NodeIndex)
		<< AnimNodeValueInt.Value(InValue)
		<< AnimNodeValueInt.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueInt.Key(InKey);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, float InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimNodeValueFloat, AnimationChannel)
		<< AnimNodeValueFloat.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueFloat.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueFloat.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueFloat.NodeId(NodeIndex)
		<< AnimNodeValueFloat.Value(InValue)
		<< AnimNodeValueFloat.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueFloat.Key(InKey);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const FVector2D& InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimNodeValueVector2D, AnimationChannel)
		<< AnimNodeValueVector2D.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueVector2D.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueVector2D.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueVector2D.NodeId(NodeIndex)
		<< AnimNodeValueVector2D.ValueX(InValue.X)
		<< AnimNodeValueVector2D.ValueY(InValue.Y)
		<< AnimNodeValueVector2D.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueVector2D.Key(InKey);
}


void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const FRotator& Value)
{
	const FVector VectorValue(Value.Roll, Value.Pitch, Value.Yaw);
	OutputAnimNodeValue(InContext, NodeIndex, InKey, VectorValue);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const FVector& InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimNodeValueVector, AnimationChannel)
		<< AnimNodeValueVector.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueVector.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueVector.AnimInstanceId(FObjectTrace::GetObjectId(InContext.AnimInstanceProxy->GetAnimInstanceObject()))
		<< AnimNodeValueVector.NodeId(NodeIndex)
		<< AnimNodeValueVector.ValueX(InValue.X)
		<< AnimNodeValueVector.ValueY(InValue.Y)
		<< AnimNodeValueVector.ValueZ(InValue.Z)
		<< AnimNodeValueVector.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueVector.Key(InKey);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const FName& InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	TCHAR Value[FName::StringBufferSize];
	uint32 ValueLength = InValue.ToString(Value);

	UE_TRACE_LOG(Animation, AnimNodeValueString, AnimationChannel)
		<< AnimNodeValueString.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueString.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueString.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueString.NodeId(NodeIndex)
		<< AnimNodeValueString.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueString.Key(InKey)
		<< AnimNodeValueString.Value(Value, ValueLength);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const TCHAR* InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimNodeValueString, AnimationChannel)
		<< AnimNodeValueString.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueString.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueString.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueString.NodeId(NodeIndex)
		<< AnimNodeValueString.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueString.Key(InKey)
		<< AnimNodeValueString.Value(InValue);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const UObject* InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);
	TRACE_OBJECT(InValue);

	UE_TRACE_LOG(Animation, AnimNodeValueObject, AnimationChannel)
		<< AnimNodeValueObject.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueObject.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueObject.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueObject.NodeId(NodeIndex)
		<< AnimNodeValueObject.Value(FObjectTrace::GetObjectId(InValue))
		<< AnimNodeValueObject.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueObject.Key(InKey);
}

void FAnimTrace::OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const UClass* InValue)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);
	TRACE_TYPE(InValue);

	UE_TRACE_LOG(Animation, AnimNodeValueClass, AnimationChannel)
		<< AnimNodeValueClass.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueClass.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueClass.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueClass.NodeId(NodeIndex)
		<< AnimNodeValueClass.Value(FObjectTrace::GetObjectId(InValue))
		<< AnimNodeValueClass.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueClass.Key(InKey);
}

void FAnimTrace::OutputAnimNodeValueAnimNode(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, int32 InValue, const UObject* InValueAnimInstanceId)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, AnimNodeValueAnimNode, AnimationChannel)
		<< AnimNodeValueAnimNode.Cycle(FPlatformTime::Cycles64())
		<< AnimNodeValueAnimNode.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< AnimNodeValueAnimNode.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimNodeValueAnimNode.NodeId(NodeIndex)
		<< AnimNodeValueAnimNode.Value(InValue)
		<< AnimNodeValueAnimNode.ValueAnimInstanceId(FObjectTrace::GetObjectId(InValueAnimInstanceId))
		<< AnimNodeValueAnimNode.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(AnimInstance))
		<< AnimNodeValueAnimNode.Key(InKey);
}

void FAnimTrace::OutputAnimSequencePlayer(const FAnimationBaseContext& InContext, const FAnimNode_SequencePlayerBase& InNode)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UAnimSequenceBase* Sequence = InNode.GetSequence();

	UE_TRACE_LOG(Animation, AnimSequencePlayer, AnimationChannel)
		<< AnimSequencePlayer.Cycle(FPlatformTime::Cycles64())
		<< AnimSequencePlayer.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< AnimSequencePlayer.NodeId(InContext.GetCurrentNodeId())
		<< AnimSequencePlayer.Position(InNode.GetAccumulatedTime())
		<< AnimSequencePlayer.Length(Sequence ? Sequence->GetPlayLength() : 0.0f)
		<< AnimSequencePlayer.FrameCounter(Sequence ? Sequence->GetNumberOfSampledKeys() : 0);
}

void FAnimTrace::OutputStateMachineState(const FAnimationBaseContext& InContext, int32 InStateMachineIndex, int32 InStateIndex, float InStateWeight, float InElapsedTime)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	if (CANNOT_TRACE_OBJECT(InContext.AnimInstanceProxy->GetSkelMeshComponent()))
	{
		return;
	}

	UObject* AnimInstance = InContext.AnimInstanceProxy->GetAnimInstanceObject();
	TRACE_OBJECT(AnimInstance);

	UE_TRACE_LOG(Animation, StateMachineState, AnimationChannel)
		<< StateMachineState.Cycle(FPlatformTime::Cycles64())
		<< StateMachineState.AnimInstanceId(FObjectTrace::GetObjectId(AnimInstance))
		<< StateMachineState.NodeId(InContext.GetCurrentNodeId())
		<< StateMachineState.StateMachineIndex(InStateMachineIndex)
		<< StateMachineState.StateIndex(InStateIndex)
		<< StateMachineState.StateWeight(InStateWeight)
		<< StateMachineState.ElapsedTime(InElapsedTime);
}

void FAnimTrace::OutputAnimNotify(UAnimInstance* InAnimInstance, const FAnimNotifyEvent& InNotifyEvent, ENotifyEventType InEventType)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}
	
	if (CANNOT_TRACE_OBJECT(InAnimInstance->GetSkelMeshComponent()))
	{
		return;
	}

	TRACE_OBJECT(InAnimInstance);

	const UObject* NotifyObject = nullptr;
	const UObject* NotifyAsset = nullptr;
	if(InNotifyEvent.Notify)
	{
		NotifyObject = InNotifyEvent.Notify;
		NotifyAsset = NotifyObject->GetOuter();
	}
	else if(InNotifyEvent.NotifyStateClass)
	{
		NotifyObject = InNotifyEvent.NotifyStateClass;
		NotifyAsset = NotifyObject->GetOuter();
	}
	else
	{
		NotifyAsset = InNotifyEvent.GetLinkedMontage();
		if (NotifyAsset == nullptr)
		{
			NotifyAsset = InNotifyEvent.GetLinkedSequence();
		}
	}
	

	TRACE_OBJECT(NotifyAsset);
	TRACE_OBJECT(NotifyObject);

	const uint32 NameId = OutputName(InNotifyEvent.NotifyName);

	UE_TRACE_LOG(Animation, Notify2, AnimationChannel)
		<< Notify2.Cycle(FPlatformTime::Cycles64())
		<< Notify2.RecordingTime(FObjectTrace::GetWorldElapsedTime(InAnimInstance->GetWorld()))
		<< Notify2.AnimInstanceId(FObjectTrace::GetObjectId(InAnimInstance))
		<< Notify2.AssetId(FObjectTrace::GetObjectId(NotifyAsset))
		<< Notify2.NotifyId(FObjectTrace::GetObjectId(NotifyObject))
		<< Notify2.NameId(NameId)
		<< Notify2.Time(InNotifyEvent.GetTime())
		<< Notify2.Duration(InNotifyEvent.GetDuration())
		<< Notify2.NotifyEventType((uint8)InEventType);
}

void FAnimTrace::OutputAnimSyncMarker(UAnimInstance* InAnimInstance, const FPassedMarker& InPassedSyncMarker)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(InAnimInstance->GetSkelMeshComponent()))
	{
		return;
	}

	TRACE_OBJECT(InAnimInstance);

	const uint32 NameId = OutputName(InPassedSyncMarker.PassedMarkerName);

	UE_TRACE_LOG(Animation, SyncMarker2, AnimationChannel)
		<< SyncMarker2.Cycle(FPlatformTime::Cycles64())
		<< SyncMarker2.AnimInstanceId(FObjectTrace::GetObjectId(InAnimInstance))
		<< SyncMarker2.RecordingTime(FObjectTrace::GetWorldElapsedTime(InAnimInstance->GetWorld()))
		<< SyncMarker2.NameId(NameId);
}

void FAnimTrace::OutputMontage(UAnimInstance* InAnimInstance, const FAnimMontageInstance& InMontageInstance)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	if(InMontageInstance.Montage != nullptr)
	{
		if (CANNOT_TRACE_OBJECT(InAnimInstance->GetSkelMeshComponent()))
		{
			return;
		}

		TRACE_OBJECT(InAnimInstance);
		TRACE_OBJECT(InMontageInstance.Montage);

		const uint32 CurrentSectionNameId = OutputName(InMontageInstance.GetCurrentSection());
		const uint32 NextSectionNameId = OutputName(InMontageInstance.GetNextSection());

		UE_TRACE_LOG(Animation, Montage2, AnimationChannel)
			<< Montage2.Cycle(FPlatformTime::Cycles64())
			<< Montage2.RecordingTime(FObjectTrace::GetWorldElapsedTime(InAnimInstance->GetWorld()))
			<< Montage2.AnimInstanceId(FObjectTrace::GetObjectId(InAnimInstance))
			<< Montage2.MontageId(FObjectTrace::GetObjectId(InMontageInstance.Montage))
			<< Montage2.CurrentSectionNameId(CurrentSectionNameId)
			<< Montage2.NextSectionNameId(NextSectionNameId)
			<< Montage2.Weight(InMontageInstance.GetWeight())
			<< Montage2.DesiredWeight(InMontageInstance.GetDesiredWeight())
			<< Montage2.Position(InMontageInstance.GetPosition())
			<< Montage2.FrameCounter(FObjectTrace::GetObjectWorldTickCounter(InAnimInstance));
	}
}

void FAnimTrace::OutputSync(const FAnimInstanceProxy& InSourceProxy, int32 InSourceNodeId, FName InGroupName)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}


	if (CANNOT_TRACE_OBJECT(InSourceProxy.GetSkelMeshComponent()))
	{
		return;
	}

	TRACE_OBJECT(InSourceProxy.GetAnimInstanceObject());

	uint32 GroupNameId = FAnimTrace::OutputName(InGroupName);

	UE_TRACE_LOG(Animation, Sync, AnimationChannel)
		<< Sync.Cycle(FPlatformTime::Cycles64())
		<< Sync.AnimInstanceId(FObjectTrace::GetObjectId(InSourceProxy.GetAnimInstanceObject()))
		<< Sync.SourceNodeId(InSourceNodeId)
		<< Sync.GroupNameId(GroupNameId);
}

void FAnimTrace::OutputPoseWatch(const FAnimInstanceProxy& InSourceProxy, UPoseWatchPoseElement* InPoseWatchElement, int32 InPoseWatchId, const TArray<FTransform>& BoneTransforms, const FBlendedHeapCurve& InCurves, const TArray<FBoneIndexType>& RequiredBones, const FTransform& WorldTransform, const bool bIsEnabled)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(InSourceProxy.GetSkelMeshComponent()))
	{
		return;
	}

	const UAnimInstance* AnimInstance = CastChecked<UAnimInstance>(InSourceProxy.GetAnimInstanceObject());
	const USkeletalMeshComponent* Component = CastChecked<USkeletalMeshComponent>(AnimInstance->GetSkelMeshComponent());

	TRACE_OBJECT(Component);
	TRACE_OBJECT(AnimInstance);

	TArray<float>& CurveValues = FAnimTraceScratchBuffers::Get().CurveValues;
	CurveValues.Reset();
	CurveValues.SetNumUninitialized(InCurves.Num());
	TArray<uint32>& CurveIds = FAnimTraceScratchBuffers::Get().CurveIds;
	CurveIds.Reset();
	CurveIds.SetNumUninitialized(InCurves.Num());

	if(InCurves.Num() > 0)
	{
		int32 CurveIndex = 0;
		InCurves.ForEachElement([&CurveValues, &CurveIds, &CurveIndex](const UE::Anim::FCurveElement& InCurveElement)
		{
			CurveIds[CurveIndex] = OutputName(InCurveElement.Name);
			CurveValues[CurveIndex] = InCurveElement.Value;
			CurveIndex++;
		});
	}
	
	UE_TRACE_LOG(Animation, PoseWatch2, AnimationChannel)
		<< PoseWatch2.Cycle(FPlatformTime::Cycles64())
		<< PoseWatch2.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
		<< PoseWatch2.ComponentId(FObjectTrace::GetObjectId(Component))
		<< PoseWatch2.AnimInstanceId(FObjectTrace::GetObjectId(InSourceProxy.GetAnimInstanceObject()))
		<< PoseWatch2.PoseWatchId(InPoseWatchId)
#if WITH_EDITOR
		<< PoseWatch2.NameId(OutputName(*InPoseWatchElement->GetParent()->GetLabel().ToString()))
		<< PoseWatch2.Color(InPoseWatchElement->GetColor().DWColor())
#endif
		<< PoseWatch2.WorldTransform(reinterpret_cast<const float*>(&WorldTransform), sizeof(FTransform) / sizeof(float))
		<< PoseWatch2.BoneTransforms(reinterpret_cast<const float*>(BoneTransforms.GetData()), BoneTransforms.Num() * (sizeof(FTransform) / sizeof(float)))
		<< PoseWatch2.CurveValues(CurveValues.GetData(), CurveValues.Num())
		<< PoseWatch2.CurveIds(CurveIds.GetData(), CurveIds.Num())
		<< PoseWatch2.RequiredBones(RequiredBones.GetData(), RequiredBones.Num())
		<< PoseWatch2.bIsEnabled(bIsEnabled);
}

void FAnimTrace::OutputInertialization(const FAnimInstanceProxy& InSourceProxy, int32 NodeId, float Weight, EInertializationType Type)
{
	const UAnimInstance* AnimInstance = CastChecked<UAnimInstance>(InSourceProxy.GetAnimInstanceObject());
	TRACE_OBJECT(AnimInstance);
	
	UE_TRACE_LOG(Animation, Inertialization, AnimationChannel)
	<< Inertialization.Cycle(FPlatformTime::Cycles64())
	<< Inertialization.RecordingTime(FObjectTrace::GetWorldElapsedTime(AnimInstance->GetWorld()))
	<< Inertialization.AnimInstanceId(FObjectTrace::GetObjectId(InSourceProxy.GetAnimInstanceObject()))
	<< Inertialization.NodeId(NodeId)
	<< Inertialization.Weight(Weight)
	<< Inertialization.Type(static_cast<uint8>(Type));
}

#endif
