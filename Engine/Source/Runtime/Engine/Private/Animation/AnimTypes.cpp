// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "AnimationUtils.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/MirrorDataTable.h"
#include "UObject/AnimObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimTypes)

#define NOTIFY_TRIGGER_OFFSET UE_KINDA_SMALL_NUMBER

float GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::Type OffsetType)
{
	switch (OffsetType)
	{
	case EAnimEventTriggerOffsets::OffsetBefore:
		{
			return -NOTIFY_TRIGGER_OFFSET;
		}
	case EAnimEventTriggerOffsets::OffsetAfter:
		{
			return NOTIFY_TRIGGER_OFFSET;
		}
	case EAnimEventTriggerOffsets::NoOffset:
		{
			return 0.f;
		}
	default:
		{
			check(false); // Unknown value supplied for OffsetType
			break;
		}
	}
	return 0.f;
}

/////////////////////////////////////////////////////
// FAnimNotifyEvent

#if WITH_EDITORONLY_DATA

bool FAnimNotifyEvent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	return false;
}

void FAnimNotifyEvent::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (!Guid.IsValid())
		{
			// Create a new Guid if this one is invalid
			Guid = FGuid::NewGuid();
		}
	}
}

#endif

void FAnimNotifyEvent::RefreshTriggerOffset(EAnimEventTriggerOffsets::Type PredictedOffsetType)
{
	if(PredictedOffsetType == EAnimEventTriggerOffsets::NoOffset || TriggerTimeOffset == 0.f)
	{
		TriggerTimeOffset = GetTriggerTimeOffsetForType(PredictedOffsetType);
	}
}

void FAnimNotifyEvent::RefreshEndTriggerOffset( EAnimEventTriggerOffsets::Type PredictedOffsetType )
{
	if(PredictedOffsetType == EAnimEventTriggerOffsets::NoOffset || EndTriggerTimeOffset == 0.f)
	{
		EndTriggerTimeOffset = GetTriggerTimeOffsetForType(PredictedOffsetType);
	}
}

float FAnimNotifyEvent::GetTriggerTime() const
{
	return GetTime() + TriggerTimeOffset;
}

float FAnimNotifyEvent::GetEndTriggerTime() const
{
	if (!NotifyStateClass && (EndTriggerTimeOffset != 0.f))
	{
		UE_LOGF(LogAnimNotify, Log, "Anim Notify %ls is non state, but has an EndTriggerTimeOffset %f!", *NotifyName.ToString(), EndTriggerTimeOffset);
	}

	return NotifyStateClass ? (GetTriggerTime() + GetDuration() + EndTriggerTimeOffset) : GetTriggerTime();
}

float FAnimNotifyEvent::GetDuration() const
{
	return NotifyStateClass ? EndLink.GetTime() - GetTime() : 0.0f;
}

void FAnimNotifyEvent::SetDuration(float NewDuration)
{
	Duration = NewDuration;
	EndLink.SetTime(GetTime() + Duration);
}

bool FAnimNotifyEvent::IsBranchingPoint() const
{
	return GetLinkedMontage() && ((MontageTickType == EMontageNotifyTickType::BranchingPoint) || (Notify && Notify->bIsNativeBranchingPoint) || (NotifyStateClass && NotifyStateClass->bIsNativeBranchingPoint));
}

void FAnimNotifyEvent::SetTime(float NewTime, EAnimLinkMethod::Type ReferenceFrame /*= EAnimLinkMethod::Absolute*/)
{
	FAnimLinkableElement::SetTime(NewTime, ReferenceFrame);
	SetDuration(Duration);
}

FName FAnimNotifyEvent::GetNotifyEventName() const
{
	if(NotifyName != NAME_None)
	{
		if(NotifyName != CachedNotifyEventBaseName)
		{
			const FString EventName = FString::Printf(TEXT("AnimNotify_%s"), *NotifyName.ToString());
			CachedNotifyEventBaseName = NotifyName;
			CachedNotifyEventName = FName(*EventName);
		}

		return CachedNotifyEventName;
	}

	return NAME_None;
}

FName FAnimNotifyEvent::GetNotifyEventName(const UMirrorDataTable* MirrorDataTable) const
{
	if (MirrorDataTable)
	{
		if(NotifyName == NAME_None)
		{
			return NAME_None;
		}
		const FName* MirroredName = MirrorDataTable->AnimNotifyToMirrorAnimNotifyMap.Find(NotifyName);
		if (MirroredName)
		{
			const FString EventName = FString::Printf(TEXT("AnimNotify_%s"), *MirroredName->ToString());
			return FName(*EventName);
		}
	}
	return GetNotifyEventName(); 
}

////////////////////////////
//
// FAnimSyncMarker
// 
////////////////////////////

#if WITH_EDITORONLY_DATA
bool FAnimSyncMarker::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	
	UScriptStruct* Struct = FAnimSyncMarker::StaticStruct();
	
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

	if (Ar.IsLoading())
	{
		if(!Guid.IsValid())
		{
			// Create a new Guid if this one is invalid
			Guid = FGuid::NewGuid();
		}
	}

	return true;
}
#endif

////////////////////////////
//
// FMarkerSyncData
// 
////////////////////////////

void FMarkerSyncData::GetMarkerIndicesForTime(float CurrentTime, bool bLooping, const TArray<FName>& ValidMarkerNames, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker, float SequenceLength) const
{
	const int LoopModStart = bLooping ? -1 : 0;
	const int LoopModEnd = bLooping ? 2 : 1;
	
	OutPrevMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
	OutPrevMarker.TimeToMarker = -CurrentTime;
	OutNextMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
	OutNextMarker.TimeToMarker = SequenceLength - CurrentTime;

	for (int32 LoopMod = LoopModStart; LoopMod < LoopModEnd; ++LoopMod)
	{
		const float LoopModTime = LoopMod * SequenceLength;
		for (int Idx = 0; Idx < AuthoredSyncMarkers.Num(); ++Idx)
		{
			const FAnimSyncMarker& Marker = AuthoredSyncMarkers[Idx];
			if (ValidMarkerNames.Contains(Marker.MarkerName))
			{
				const float MarkerTime = Marker.Time + LoopModTime;
				if (MarkerTime < CurrentTime)
				{
					OutPrevMarker.MarkerIndex = Idx;
					OutPrevMarker.TimeToMarker = MarkerTime - CurrentTime;
				}
				else if (MarkerTime >= CurrentTime)
				{
					OutNextMarker.MarkerIndex = Idx;
					OutNextMarker.TimeToMarker = MarkerTime - CurrentTime;
					break; // Done
				}
			}
		}

		// Continue looking for an authored next sync marker.
		if (OutNextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary)
		{
			break; // Done
		}
	}
}

FMarkerSyncAnimPosition FMarkerSyncData::GetMarkerSyncPositionFromMarkerIndicies(int32 PrevMarker, int32 NextMarker, float CurrentTime, float SequenceLength, const UMirrorDataTable* MirrorTable) const
{
	FMarkerSyncAnimPosition SyncPosition;
	float PrevTime, NextTime;

	// Get previous marker's time and name.
	if (PrevMarker != MarkerIndexSpecialValues::AnimationBoundary && AuthoredSyncMarkers.IsValidIndex(PrevMarker))
	{
		PrevTime = AuthoredSyncMarkers[PrevMarker].Time;
		SyncPosition.PreviousMarkerName = AuthoredSyncMarkers[PrevMarker].MarkerName;
		if (MirrorTable)
		{
			const FName* MirroredName = MirrorTable->SyncToMirrorSyncMap.Find(SyncPosition.PreviousMarkerName);
			if (MirroredName)
			{
				SyncPosition.PreviousMarkerName = *MirroredName;
			}
		}
	}
	else
	{
		PrevTime = 0.f;
	}

	// Get next marker's time and name.
	if (NextMarker != MarkerIndexSpecialValues::AnimationBoundary && AuthoredSyncMarkers.IsValidIndex(NextMarker))
	{
		NextTime = AuthoredSyncMarkers[NextMarker].Time;
		SyncPosition.NextMarkerName = AuthoredSyncMarkers[NextMarker].MarkerName;
		if (MirrorTable)
		{
			const FName* MirroredName = MirrorTable->SyncToMirrorSyncMap.Find(SyncPosition.NextMarkerName);
			if (MirroredName)
			{
				SyncPosition.NextMarkerName = *MirroredName;
			}
		}
	}
	else
	{
		NextTime = SequenceLength;
	}

	// Account for looping
	if (PrevTime > NextTime)
	{
		PrevTime = (PrevTime > CurrentTime) ? PrevTime - SequenceLength : PrevTime;
		NextTime = (NextTime < CurrentTime) ? NextTime + SequenceLength : NextTime;
	}
	else if (PrevTime > CurrentTime)
	{
		CurrentTime += SequenceLength;
	}
	
	if (PrevTime == NextTime)
	{
		PrevTime -= SequenceLength;
	}

	check(NextTime > PrevTime);

	// Store the encoded current time position as a ratio between markers.
	SyncPosition.PositionBetweenMarkers = (CurrentTime - PrevTime) / (NextTime - PrevTime);
	return SyncPosition;
}

void FMarkerSyncData::CollectUniqueNames()
{
	if (AuthoredSyncMarkers.Num() > 0)
	{
		AuthoredSyncMarkers.Sort();
		UniqueMarkerNames.Reset();
		UniqueMarkerNames.Reserve(AuthoredSyncMarkers.Num());

		const FAnimSyncMarker* PreviousMarker = nullptr;
		for (const FAnimSyncMarker& Marker : AuthoredSyncMarkers)
		{
			UniqueMarkerNames.AddUnique(Marker.MarkerName);
			PreviousMarker = &Marker;
		}
	}
	else
	{
		UniqueMarkerNames.Empty();
	}
}

void FMarkerSyncData::CollectMarkersInRange(float PrevPosition, float NewPosition, 	TArray<FPassedMarker>& OutMarkersPassedThisTick, float TotalDeltaMove)
{
	CollectMarkersInRange(PrevPosition, NewPosition, OutMarkersPassedThisTick, TotalDeltaMove, nullptr);
}

extern FName MarkerOrMirroredName(FName MarkerToFind, const UMirrorDataTable* MirrorTable);

void FMarkerSyncData::CollectMarkersInRange(float PrevPosition, float NewPosition, TArray<FPassedMarker>& OutMarkersPassedThisTick, float TotalDeltaMove, const UMirrorDataTable* MirrorDataTable)
{
	for (const FAnimSyncMarker& Marker : AuthoredSyncMarkers)
	{
		if (Marker.Time >= PrevPosition && Marker.Time < NewPosition)
		{
			float TimeToMarker = Marker.Time - PrevPosition;
			int32 PassedMarker = OutMarkersPassedThisTick.Add(FPassedMarker());
			OutMarkersPassedThisTick[PassedMarker].PassedMarkerName = MarkerOrMirroredName(Marker.MarkerName, MirrorDataTable);
			OutMarkersPassedThisTick[PassedMarker].DeltaTimeWhenPassed = TotalDeltaMove - TimeToMarker;
		}
	}
}

#if 0 // Debug logging
template <>
void DebugLogArray(const TArray<FRawAnimSequenceTrack>& RawData)
{
	for (int32 i = 0; i < RawData.Num(); ++i)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Track :%i\nTran\n"), i);
		DebugLogArray(RawData[i].PosKeys);
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Rot\n"));
		DebugLogArray(RawData[i].RotKeys);
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Scale\n"));
		DebugLogArray(RawData[i].ScaleKeys);
	}
}
#endif
