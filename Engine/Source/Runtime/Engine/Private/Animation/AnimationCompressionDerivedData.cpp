// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimationCompressionDerivedData.h"

#include "AnimationUtils.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimCurveCompressionSettings.h"
#include "Animation/AnimCurveCompressionCodec.h"
#include "Experimental/Misc/ExecutionResource.h"
#include "Animation/AnimationSequenceCompiler.h"
#include "Animation/AnimCompress.h"
#include "DerivedDataCache.h"
#include "Animation/Skeleton.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/ITargetPlatform.h"

namespace UE::Anim
{	
#if ENABLE_COOK_STATS
	namespace AnimSequenceCookStats
	{
		FCookStats::FDDCResourceUsageStats UsageStats;
		static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
		{
			UsageStats.LogStats(AddStat, TEXT("AnimationSequence.Usage"), TEXT(""));
		});
	}
#endif
	
#if WITH_EDITOR
FAnimationSequenceAsyncCacheTask::FAnimationSequenceAsyncCacheTask(const FIoHash& InKeyHash, FCompressibleAnimPtr InCompressibleAnimPtr, FCompressedAnimSequence* InCompressedData,
	UAnimSequence& InAnimSequence, const ITargetPlatform* InTargetPlatform)
	: Owner(UE::DerivedData::EPriority::Highest)
		, CompressedData(InCompressedData)
		, WeakAnimSequence(&InAnimSequence)
		, CompressibleAnimPtr(InCompressibleAnimPtr)
		, TargetPlatform(InTargetPlatform)
		, RequiredMemory(0)
{
	check(!InAnimSequence.IsUnreachable());
	check(!InAnimSequence.GetSkeleton()->IsUnreachable());
		
	BeginCache(InKeyHash);
}

FAnimationSequenceAsyncCacheTask::~FAnimationSequenceAsyncCacheTask()
{
	if (!Poll())
	{
		Cancel();
		Wait(false);
	}
}

void FAnimationSequenceAsyncCacheTask::Cancel()
{
	CompressibleAnimPtr->IsCancelledSignal.Cancel();		
	if (BuildTask)
	{
		BuildTask->Cancel();
	}

	Owner.Cancel();
}

void FAnimationSequenceAsyncCacheTask::Wait(bool bPerformWork /*= true*/)
{
	if (BuildTask != nullptr)
	{
		BuildTask->EnsureCompletion(bPerformWork);
	}

	Owner.Wait();
}

bool FAnimationSequenceAsyncCacheTask::WaitWithTimeout(float TimeLimitSeconds)
{
	if (BuildTask != nullptr && !BuildTask->WaitCompletionWithTimeout(TimeLimitSeconds))
	{
		return false;
	}

	return Owner.Poll();
}

bool FAnimationSequenceAsyncCacheTask::Poll() const
{
	if (BuildTask && !BuildTask->IsDone())
	{
		return false;
	}

	return Owner.Poll();
}

void FAnimationSequenceAsyncCacheTask::Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority) const
{
	if (BuildTask)
	{
		BuildTask->Reschedule(InThreadPool, InPriority);
	}
}

void FAnimationSequenceAsyncCacheTask::BeginCache(const FIoHash& KeyHash)
{
	using namespace UE::DerivedData;
	
	if (const UAnimSequence* AnimSequence = WeakAnimSequence.Get())
	{
		FQueuedThreadPool* ThreadPool = Anim::FAnimSequenceCompilingManager::Get().GetThreadPool();
		const EQueuedWorkPriority BasePriority = Anim::FAnimSequenceCompilingManager::Get().GetBasePriority(AnimSequence);
		CalculateRequiredMemoryEstimate();

		CompressionStartTime = FPlatformTime::Seconds();

		check(BuildTask == nullptr);
		BuildTask = MakeUnique<FAnimationSequenceAsyncBuildTask>(this, KeyHash);
		BuildTask->StartBackgroundTask(ThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, RequiredMemory, TEXT("AnimationCompression"));
	}
}
	
void FAnimationSequenceAsyncCacheTask::EndCache(UE::DerivedData::FCacheGetValueResponse&& Response)
{
	using namespace UE::DerivedData;
	
	if (Response.Status == EStatus::Ok)
	{
		Owner.LaunchTask(TEXT("AnimationSequenceSerialize"), [this, Name = Response.Name, Key = Response.Key, Value = MoveTemp(Response.Value)]
		{
			bool bIsDataValid = true;

			{
				// Release execution resource as soon as the task is done
				ON_SCOPE_EXIT{ if (bIsDataValid) { ExecutionResource = nullptr; } };

				if (UAnimSequence* AnimSequence = WeakAnimSequence.Get())
				{
					COOK_STAT(auto Timer = AnimSequenceCookStats::UsageStats.TimeSyncWork());

					UE_COMPRESSED_DATA_WRITE_SCOPE(AnimSequence);
					const FSharedBuffer RecordData = Value.GetData().Decompress();
					FMemoryReaderView Ar(RecordData, /*bIsPersistent*/ true);
					CompressedData->SerializeCompressedData(Ar, true, AnimSequence, AnimSequence->GetSkeleton(), CompressibleAnimPtr->BoneCompressionSettings, CompressibleAnimPtr->CurveCompressionSettings);

					if (!CompressedData->IsValid(AnimSequence, true))
					{
						UE_LOGF(LogAnimationCompression, Warning, "Fetched invalid compressed animation data for %ls", *CompressibleAnimPtr->FullName);
						CompressedData->Reset();

						bIsDataValid = false;
					}
					else
					{
						UE_LOGF(LogAnimationCompression, Verbose, "Fetched compressed animation data for %ls", *CompressibleAnimPtr->FullName);
						COOK_STAT(Timer.AddHit(int64(Ar.TotalSize())));
					}

					if (Compression::FAnimationCompressionMemorySummaryScope::ShouldStoreCompressionResults())
					{
						const double CompressionEndTime = FPlatformTime::Seconds();
						const double CompressionTime = CompressionEndTime - CompressionStartTime;

						TArray<FBoneData> BoneData;
						FAnimationUtils::BuildSkeletonMetaData(AnimSequence->GetSkeleton(), BoneData);
						Compression::FAnimationCompressionMemorySummaryScope::CompressionResultSummary().GatherPostCompressionStats(*CompressedData, BoneData, AnimSequence->GetFName(), CompressionTime, false);
					}
				}
			}

			if (!bIsDataValid)
			{
				// Our DDC data appears to be corrupted, launch a new compression task to refresh it
				LaunchCompressionTask(Name, Key);
			}
		});
	}
	else if (Response.Status == EStatus::Error)
	{
		LaunchCompressionTask(Response.Name, Response.Key);
	}
	else
	{
		// Release execution resource as soon as the task is done
		ExecutionResource = nullptr;
	}
}

bool FAnimationSequenceAsyncCacheTask::BuildData() const
{	
	// This is where we should do the compression parts
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*(FString(TEXT("FAnimationSequenceAsyncCacheTask::BuildData ") + CompressibleAnimPtr->Name)));

	// Early out before logging if we are canceled (could be retracting this task)
	if (Owner.IsCanceled())
	{
		return false;
	}

	UE_LOGF(LogAnimationCompression, Display, "Building compressed animation data for %ls (Required Memory Estimate: %.2f MB)",
		*CompressibleAnimPtr->FullName, double(RequiredMemory) / (1024.0 * 1024.0));

	check(CompressibleAnimPtr.IsValid());
	FCompressibleAnimData& DataToCompress = *CompressibleAnimPtr.Get();
	FCompressedAnimSequence& OutData = *CompressedData;

	if (Owner.IsCanceled())
	{
		return false;
	}
	
	FCompressibleAnimDataResult CompressionResult;
	if (DataToCompress.FetchData(TargetPlatform))
	{
		if (Owner.IsCanceled())
		{
			return false;
		}

		const FString BoneCompressionName = DataToCompress.BoneCompressionSettings ? DataToCompress.BoneCompressionSettings->GetFullName() : TEXT("NONE");
		const FString CurveCompressionName = DataToCompress.CurveCompressionSettings ? DataToCompress.CurveCompressionSettings->GetFullName() : TEXT("NONE");

		const UAnimSequence* AnimSequence = WeakAnimSequence.Get();
		if (!Owner.IsCanceled() && AnimSequence)
		{
			UE_COMPRESSED_DATA_WRITE_SCOPE(AnimSequence);

			// All compression and OutData mutations happen under the write lock
			// to prevent readers from seeing partial/intermediate state
			const bool bBoneCompressionOk = FAnimationUtils::CompressAnimBones(DataToCompress, CompressionResult);
			if (Owner.IsCanceled())
			{
				return false;
			}

			const bool bCurveCompressionOk = FAnimationUtils::CompressAnimCurves(DataToCompress, OutData);
			if (bBoneCompressionOk && bCurveCompressionOk && !Owner.IsCanceled())
			{
				DataToCompress.Update(OutData);
				OutData.CompressedByteStream = MoveTemp(CompressionResult.CompressedByteStream);
				OutData.CompressedDataStructure = MoveTemp(CompressionResult.AnimData);
				OutData.BoneCompressionCodec = CompressionResult.Codec;
				OutData.CompressedRawData = DataToCompress.RawAnimationData;
				OutData.OwnerName = DataToCompress.AnimFName;

				return true;
			}
			else if (!bBoneCompressionOk || !bCurveCompressionOk)
			{
				UE_LOGF(LogAnimationCompression, Error, "Failed to generate compressed animation data for %ls with bone compression scheme %ls [%i] and curve compression scheme %ls [%i] for target platform %ls", *CompressibleAnimPtr->FullName, *BoneCompressionName, bBoneCompressionOk ? 1 : 0, *CurveCompressionName, bCurveCompressionOk ? 1 : 0, *TargetPlatform->DisplayName().ToString());
			}
		}
		else if (!AnimSequence)
		{
			UE_LOGF(LogAnimationCompression, Warning, "Failed to apply compressed animation data for %ls as it has previously been garbage collected", *CompressibleAnimPtr->FullName);
		}
	}
	else
	{
		UE_LOGF(LogAnimationCompression, Log, "Failed to fetch FCompressibleAnimData data for %ls", *CompressibleAnimPtr->FullName);
	}

	return false;
}

void FAnimationSequenceAsyncCacheTask::LaunchCompressionTask(const UE::FSharedString& Name, const UE::DerivedData::FCacheKey& Key)
{
	Owner.LaunchTask(TEXT("AnimationSequenceCompression"), [this, Name, Key]
	{
		COOK_STAT(auto Timer = AnimSequenceCookStats::UsageStats.TimeSyncWork());

		// Release execution resource as soon as the task is done
		ON_SCOPE_EXIT{ ExecutionResource = nullptr; };

		if (!BuildData())
		{
			return;
		}

		if (const UAnimSequence* AnimSequence = WeakAnimSequence.Get())
		{
			if (!CompressedData->IsBoneDataValid(AnimSequence, true) || !CompressedData->IsCurveDataValid(AnimSequence, true))
			{
				CompressedData->Reset();
				UE_LOGF(LogAnimationCompression, Warning, "Generated invalid compressed animation data for %ls", *CompressibleAnimPtr->FullName);
			}
			else
			{
				CompressedData->bHasValidCompressedData = true;

				UE_COMPRESSED_DATA_READ_SCOPE(AnimSequence);
				TArray64<uint8> RecordData;
				FMemoryWriter64 Ar(RecordData, /*bIsPersistent*/ true);
				CompressedData->SerializeCompressedData(Ar, true, const_cast<UAnimSequence*>(AnimSequence), nullptr, CompressibleAnimPtr->BoneCompressionSettings, CompressibleAnimPtr->CurveCompressionSettings);
				UE_LOGF(LogAnimationCompression, Display, "Storing compressed animation data for %ls, at %ls/%ls", *Name, *FString(Key.Bucket.ToString()), *LexToString(Key.Hash));
				UE::DerivedData::GetCache().PutValue({ {Name, Key, UE::DerivedData::FValue::Compress(MakeSharedBufferFromArray(MoveTemp(RecordData)))} }, Owner);

				COOK_STAT(Timer.AddMiss(int64(Ar.Tell())));
			}

			if (Compression::FAnimationCompressionMemorySummaryScope::ShouldStoreCompressionResults())
			{
				const double CompressionEndTime = FPlatformTime::Seconds();
				const double CompressionTime = CompressionEndTime - CompressionStartTime;
				Compression::FAnimationCompressionMemorySummaryScope::CompressionResultSummary().GatherPostCompressionStats(*CompressedData, CompressibleAnimPtr->BoneData, AnimSequence->GetFName(), CompressionTime, true);
			}
		}
	});
}

void FAnimationSequenceAsyncCacheTask::CalculateRequiredMemoryEstimate()
{
	RequiredMemory = 0;
	if (const UAnimSequence* AnimSequence = WeakAnimSequence.Get())
	{
		// Includes bone/curves
		const int64 AdditiveAnimSize = !AnimSequence->IsValidAdditive() ? 0 : [AnimSequence]()
			{
				if (const UAnimSequence* RefPoseSeq = AnimSequence->RefPoseSeq)
				{
					return RefPoseSeq->GetApproxRawSize();
				}

				return AnimSequence->GetApproxRawSize();
			}();

		// Includes bone/curves
		RequiredMemory = AnimSequence->GetApproxRawSize() + AdditiveAnimSize;
		if (Compression::FAnimationCompressionMemorySummaryScope::ShouldStoreCompressionResults())
		{
			Compression::FAnimationCompressionMemorySummaryScope::CompressionResultSummary().GatherPreCompressionStats(AnimSequence->GetApproxRawSize(), AnimSequence->GetApproxCompressedSize_Lockless());
		}

		if (const UAnimBoneCompressionSettings* BoneCompressionSettings = AnimSequence->BoneCompressionSettings.Get())
		{
			// We try out all compression codecs in parallel to find the best one, so we need to sum up the cost for every codec.
			for (TObjectPtr<UAnimBoneCompressionCodec> Codec : BoneCompressionSettings->Codecs)
			{
				if (Codec)
				{
					const int64 PeakMemoryEstimate = Codec->EstimateCompressionMemoryUsage(*AnimSequence);
					if (PeakMemoryEstimate < 0)
					{
						// Assume the worst and default to the default behavior when no estimate is given.
						UE_LOGF(LogAnimationCompression, Warning, "Got invalid memory usage estimate from codec %ls for %ls. This can negatively affect the time compression takes.", *Codec->GetFullName(), *AnimSequence->GetFullName());
						RequiredMemory = -1;
						break;
					}

					RequiredMemory += PeakMemoryEstimate;
				}
			}
		}

		// Curve compression occurs after bone compression and thus executes in its shadow (memory peak wise)
		// but it can still be the case that a sequence has no bone data while having curve data
		if (const UAnimCurveCompressionSettings* CurveCompressionSettings = AnimSequence->CurveCompressionSettings.Get())
		{
			if (CurveCompressionSettings->Codec)
			{
				const int64 PeakMemoryEstimate = CurveCompressionSettings->Codec->EstimateCompressionMemoryUsage(*AnimSequence);
				if (PeakMemoryEstimate < 0)
				{
					// Assume the worst and default to the default behavior when no estimate is given.
					UE_LOGF(LogAnimationCompression, Warning, "Got invalid memory usage estimate from codec %ls for %ls. This can negatively affect the time compression takes.", *CurveCompressionSettings->Codec->GetFullName(), *AnimSequence->GetFullName());
					RequiredMemory = -1;
				}
				else
				{
					RequiredMemory += PeakMemoryEstimate;
				}
			}
		}
	}
}

int32 GSkipDDC = 0;
static FAutoConsoleVariableRef CVarSkipDDC(
	TEXT("a.SkipDDC"),
	GSkipDDC,
	TEXT("1 = Skip DDC during compression. 0 = Include DDC results during compression "));

void FAnimationSequenceAsyncBuildWorker::DoWork() const
{
	using namespace UE::DerivedData;
	if (const UAnimSequence* AnimSequence = Owner->WeakAnimSequence.Get())
	{
		// Grab any execution resources currently assigned to this worker so that we maintain
		// concurrency limit and memory pressure until the whole multi-step task is done.
		Owner->ExecutionResource = FExecutionResourceContext::Get();

		const ECachePolicy Policy = GSkipDDC ? ECachePolicy::None : ECachePolicy::Default;
		static const FCacheBucket Bucket("AnimationSequence");
		GetCache().GetValue({ {{AnimSequence->GetPathName()}, {Bucket, IoHash}, Policy} }, Owner->Owner,
			  [Task = Owner](FCacheGetValueResponse&& Response) { Task->EndCache(MoveTemp(Response)); });
	}
}
#endif // WITH_EDITOR
}
