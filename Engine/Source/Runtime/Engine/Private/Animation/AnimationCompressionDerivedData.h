// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITORONLY_DATA
#include "DerivedDataRequestOwner.h"
#include "Containers/StringFwd.h"
#include "Experimental/Misc/ExecutionResource.h"
#include "Async/AsyncWork.h"
#include "Animation/AnimCompressionTypes.h"
#include "IO/IoHash.h"
#endif // WITH_EDITORONLY_DATA

#include "ProfilingDebugging/CookStats.h"

struct FCompressedAnimSequence;
class UAnimSequence;
namespace UE::DerivedData
{
	struct FCacheGetValueResponse;
	struct FCacheKey;
}

namespace UE::Anim
{
#if ENABLE_COOK_STATS
	namespace AnimSequenceCookStats
	{
		extern FCookStats::FDDCResourceUsageStats UsageStats;
	}
#endif

#if WITH_EDITOR
	class FAnimationSequenceAsyncCacheTask;

	class FAnimationSequenceAsyncBuildWorker : public FNonAbandonableTask
	{
		FAnimationSequenceAsyncCacheTask* Owner;
		FIoHash IoHash;
	public:
		FAnimationSequenceAsyncBuildWorker(
			FAnimationSequenceAsyncCacheTask* InOwner,
			const FIoHash& InIoHash)
			: Owner(InOwner)
			, IoHash(InIoHash)
		{
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAnimationSequenceAsyncBuildWorker, STATGROUP_ThreadPoolAsyncTasks);
		}

		void DoWork() const;
	};

	struct FAnimationSequenceAsyncBuildTask : public FAsyncTask<FAnimationSequenceAsyncBuildWorker>
	{
		FAnimationSequenceAsyncBuildTask(
			FAnimationSequenceAsyncCacheTask* InOwner,
			const FIoHash& InIoHash)
			: FAsyncTask<FAnimationSequenceAsyncBuildWorker>(InOwner, InIoHash)
		{
		}
	};
	
	class FAnimationSequenceAsyncCacheTask
	{
	public:
		FAnimationSequenceAsyncCacheTask(const FIoHash& InKeyHash,
			FCompressibleAnimPtr InCompressibleAnimPtr,
			FCompressedAnimSequence* InCompressedData,
			UAnimSequence& InAnimSequence,
			const ITargetPlatform* InTargetPlatform);

		~FAnimationSequenceAsyncCacheTask();

		void Cancel();
		void Wait(bool bPerformWork = true);
		bool WaitWithTimeout(float TimeLimitSeconds);
		bool Poll() const;
		void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority) const;
		bool WasCancelled() const { return CompressibleAnimPtr->IsCancelled() || Owner.IsCanceled(); }
		FCompressedAnimSequence* GetTargetCompressedData() const { return CompressedData; }
		const ITargetPlatform* GetTargetPlatform() const { return TargetPlatform; }
	private:
		void BeginCache(const FIoHash& KeyHash);
		void EndCache(DerivedData::FCacheGetValueResponse&& Response);
		bool BuildData() const;
		void LaunchCompressionTask(const FSharedString& Name, const DerivedData::FCacheKey& Key);
		void CalculateRequiredMemoryEstimate();

	private:
		friend class FAnimationSequenceAsyncBuildWorker;
		DerivedData::FRequestOwner Owner;

		TRefCountPtr<IExecutionResource> ExecutionResource;
		TUniquePtr<FAnimationSequenceAsyncBuildTask> BuildTask;
		FCompressedAnimSequence* CompressedData;
		TWeakObjectPtr<UAnimSequence> WeakAnimSequence;
		FCompressibleAnimPtr CompressibleAnimPtr;
		const ITargetPlatform* TargetPlatform;
		double CompressionStartTime;
		int64 RequiredMemory;
	};
#endif
}
