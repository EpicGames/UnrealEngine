// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimationDecompression.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "Animation/AnimCompressionTypes.h"
#include "Animation/AnimBoneCompressionCodec.h"

#include "AnimationRuntime.h"
#include "AnimEncoding.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/SkeletonRemapping.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);
DECLARE_CYCLE_STAT(TEXT("Build Anim Track Pairs"), STAT_BuildAnimTrackPairs, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("Extract Pose From Anim Data"), STAT_ExtractPoseFromAnimData, STATGROUP_Anim);

namespace UE::Anim::Decompression {
	
struct FGetBonePoseScratchArea : public TThreadSingleton<FGetBonePoseScratchArea>
{
	BoneTrackArray RotationScalePairs;
	BoneTrackArray TranslationPairs;
	BoneTrackArray AnimScaleRetargetingPairs;
	BoneTrackArray AnimRelativeRetargetingPairs;
	BoneTrackArray OrientAndScaleRetargetingPairs;

	// A bit set that specifies whether a compact bone index has its rotation animated by the sequence or not
	TBitArray<> AnimatedCompactRotations;
};

void DecompressPose(FCompactPose& OutPose,
								const FCompressedAnimSequence& CompressedData,
								const FAnimExtractContext& ExtractionContext,
								FAnimSequenceDecompressionContext& DecompressionContext,
								FName RetargetSource,
								const FRootMotionReset& RootMotionReset)
{
	const TArray<FTransform>& RetargetTransforms = DecompressionContext.GetSourceSkeleton()->GetRefLocalPoses(RetargetSource);
	DecompressPose(OutPose, CompressedData, ExtractionContext, DecompressionContext, RetargetTransforms, RootMotionReset);
}

void DecompressPose(FCompactPose& OutPose,
	const FCompressedAnimSequence& CompressedData,
	const FAnimExtractContext& ExtractionContext,
	FAnimSequenceDecompressionContext& DecompressionContext,
	const TArray<FTransform>& RetargetTransforms,
	const FRootMotionReset& RootMotionReset)
{
	const int32 NumCompactBones = OutPose.GetNumBones();
	const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();
	const int32 NumTracks = CompressedData.CompressedTrackToSkeletonMapTable.Num();

	const USkeleton* SourceSkeleton = DecompressionContext.GetSourceSkeleton();
	const USkeleton* TargetSkeleton = RequiredBones.GetSkeletonAsset();
	const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
	const bool bIsSkeletonRemappingValid = SkeletonRemapping.IsValid();

	FGetBonePoseScratchArea& ScratchArea = FGetBonePoseScratchArea::Get();
	BoneTrackArray& RotationScalePairs = ScratchArea.RotationScalePairs;
	BoneTrackArray& TranslationPairs = ScratchArea.TranslationPairs;
	BoneTrackArray& AnimScaleRetargetingPairs = ScratchArea.AnimScaleRetargetingPairs;
	BoneTrackArray& AnimRelativeRetargetingPairs = ScratchArea.AnimRelativeRetargetingPairs;
	BoneTrackArray& OrientAndScaleRetargetingPairs = ScratchArea.OrientAndScaleRetargetingPairs;

	// build a list of desired bones
	RotationScalePairs.Reset();
	TranslationPairs.Reset();
	AnimScaleRetargetingPairs.Reset();
	AnimRelativeRetargetingPairs.Reset();
	OrientAndScaleRetargetingPairs.Reset();

	const bool bUseSourceRetargetModes = TargetSkeleton->GetUseRetargetModesFromCompatibleSkeleton();
	const bool bDisableRetargeting = RequiredBones.GetDisableRetargeting();

	const bool bIsMeshSpaceAdditive = DecompressionContext.GetAdditiveType() == AAT_RotationOffsetMeshSpace;
	TBitArray<>& AnimatedCompactRotations = ScratchArea.AnimatedCompactRotations;
	if (bIsMeshSpaceAdditive)
	{
		AnimatedCompactRotations.Init(false, NumCompactBones);
	}

	// Optimization: assuming first index is root bone. That should always be the case in Skeletons.
	checkSlow((RequiredBones.GetMeshPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(0)) == FMeshPoseBoneIndex(0)));
	// this is not guaranteed for AnimSequences though... If Root is not animated, Track will not exist.
	const bool bFirstTrackIsRootBone = (CompressedData.GetSkeletonIndexFromTrackIndex(0) == 0);

	{
		SCOPE_CYCLE_COUNTER(STAT_BuildAnimTrackPairs);

		// Handle root bone separately if it is track 0. so we start w/ Index 1.
		for (int32 TrackIndex = (bFirstTrackIsRootBone ? 1 : 0); TrackIndex < NumTracks; TrackIndex++)
		{
			const int32 SourceSkeletonBoneIndex = CompressedData.GetSkeletonIndexFromTrackIndex(TrackIndex);
			const int32 TargetSkeletonBoneIndex = bIsSkeletonRemappingValid ? SkeletonRemapping.GetTargetSkeletonBoneIndex(SourceSkeletonBoneIndex) : SourceSkeletonBoneIndex;

			if (TargetSkeletonBoneIndex != INDEX_NONE)
			{
				const FSkeletonPoseBoneIndex SkeletonPoseBoneIndex(TargetSkeletonBoneIndex);
				const FCompactPoseBoneIndex BoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonPoseBoneIndex);
				//Nasty, we break our type safety, code in the lower levels should be adjusted for this
				const int32 CompactPoseBoneIndex = BoneIndex.GetInt();
				if (CompactPoseBoneIndex != INDEX_NONE)
				{
					RotationScalePairs.Add(BoneTrackPair(CompactPoseBoneIndex, TrackIndex));

					if (bIsMeshSpaceAdditive)
					{
						AnimatedCompactRotations[CompactPoseBoneIndex] = true;
					}

					// Check what retarget mode to use for the translational retargeting for this specific bone.
					const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
						bUseSourceRetargetModes,
						SourceSkeletonBoneIndex,
						TargetSkeletonBoneIndex,
						SourceSkeleton,
						TargetSkeleton,
						bDisableRetargeting);

					// Skip extracting translation component for EBoneTranslationRetargetingMode::Skeleton.
					switch (RetargetMode)
					{
					case EBoneTranslationRetargetingMode::Animation:
						TranslationPairs.Add(BoneTrackPair(CompactPoseBoneIndex, TrackIndex));
						break;
					case EBoneTranslationRetargetingMode::AnimationScaled:
						TranslationPairs.Add(BoneTrackPair(CompactPoseBoneIndex, TrackIndex));
						AnimScaleRetargetingPairs.Add(BoneTrackPair(CompactPoseBoneIndex, SourceSkeletonBoneIndex));
						break;
					case EBoneTranslationRetargetingMode::AnimationRelative:
						TranslationPairs.Add(BoneTrackPair(CompactPoseBoneIndex, TrackIndex));

						// With baked additives, we can skip 'AnimationRelative' tracks, as the relative transform gets canceled out.
						// (A1 + Rel) - (A2 + Rel) = A1 - A2.
						if (!DecompressionContext.IsAdditiveAnimation())
						{
							AnimRelativeRetargetingPairs.Add(BoneTrackPair(CompactPoseBoneIndex, SourceSkeletonBoneIndex));
						}
						break;
					case EBoneTranslationRetargetingMode::OrientAndScale:
						TranslationPairs.Add(BoneTrackPair(CompactPoseBoneIndex, TrackIndex));

						// Additives remain additives, they're not retargeted.
						if (!DecompressionContext.IsAdditiveAnimation())
						{
							OrientAndScaleRetargetingPairs.Add(BoneTrackPair(CompactPoseBoneIndex, SourceSkeletonBoneIndex));
						}
						break;
					}
				}
			}
		}
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_ExtractPoseFromAnimData);
		CSV_SCOPED_TIMING_STAT(Animation, ExtractPoseFromAnimData);
		CSV_CUSTOM_STAT(Animation, NumberOfExtractedAnimations, 1, ECsvCustomStatOp::Accumulate);

		DecompressionContext.Seek(ExtractionContext.CurrentTime);

		// Handle Root Bone separately
		if (bFirstTrackIsRootBone)
		{
			const int32 TrackIndex = 0;
			FCompactPoseBoneIndex RootBone(0);
			FTransform& RootAtom = OutPose[RootBone];

			CompressedData.BoneCompressionCodec->DecompressBone(DecompressionContext, TrackIndex, RootAtom);

			// Retarget the root onto the target skeleton (correcting for differences in rest poses)
			if (SkeletonRemapping.RequiresReferencePoseRetarget())
			{
				// Root bone does not require fix-up for additive animations as there is no parent delta rotation to account for
				if (!DecompressionContext.IsAdditiveAnimation())
				{
					constexpr int32 SkeletonBoneIndex = 0;

					// Check what retarget mode to use for the translational retargeting for this specific bone.
					const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
						bUseSourceRetargetModes,
						SkeletonBoneIndex,
						SkeletonBoneIndex,
						SourceSkeleton,
						TargetSkeleton,
						bDisableRetargeting);

					RootAtom.SetRotation(SkeletonRemapping.RetargetBoneRotationToTargetSkeleton(SkeletonBoneIndex, RootAtom.GetRotation()));
					if (RetargetMode != EBoneTranslationRetargetingMode::Skeleton)
					{
						RootAtom.SetTranslation(SkeletonRemapping.RetargetBoneTranslationToTargetSkeleton(SkeletonBoneIndex, RootAtom.GetTranslation()));
					}
				}
			}

			// @laurent - we should look into splitting rotation and translation tracks, so we don't have to process translation twice.
			FAnimationRuntime::RetargetBoneTransform(DecompressionContext.GetSourceSkeleton(), DecompressionContext.AnimName, RetargetTransforms, RootAtom, 0, RootBone, RequiredBones, DecompressionContext.IsAdditiveAnimation());
		}

		if (RotationScalePairs.Num() > 0)
		{
			// get the remaining bone atoms
			TArrayView<FTransform> OutPoseBones = OutPose.GetMutableBones();
			CompressedData.BoneCompressionCodec->DecompressPose(DecompressionContext, RotationScalePairs, TranslationPairs, RotationScalePairs, OutPoseBones);
		}
	}

	// Retarget the pose onto the target skeleton (correcting for differences in rest poses)
	if (SkeletonRemapping.RequiresReferencePoseRetarget())
	{
		if (DecompressionContext.IsAdditiveAnimation())
		{
			for (FCompactPoseBoneIndex BoneIndex(bFirstTrackIsRootBone ? 1 : 0); BoneIndex < NumCompactBones; ++BoneIndex)
			{
				const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = RequiredBones.GetSkeletonPoseIndexFromCompactPoseIndex(BoneIndex);
				if(!TargetSkeletonBoneIndex.IsValid())
				{
					continue;
				}

				// Mesh space additives do not require fix-up
				if (DecompressionContext.GetAdditiveType() == AAT_LocalSpaceBase)
				{
					OutPose[BoneIndex].SetRotation(SkeletonRemapping.RetargetAdditiveRotationToTargetSkeleton(TargetSkeletonBoneIndex.GetInt(), OutPose[BoneIndex].GetRotation()));
				}
			
				// Check what retarget mode to use for the translational retargeting for this specific bone.
				const int32 SourceSkeletonBoneIndex = SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt());
				const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
					bUseSourceRetargetModes,
					SourceSkeletonBoneIndex,
					TargetSkeletonBoneIndex.GetInt(),
					SourceSkeleton,
					TargetSkeleton,
					bDisableRetargeting);

				if (RetargetMode != EBoneTranslationRetargetingMode::Skeleton)
				{
					OutPose[BoneIndex].SetTranslation(SkeletonRemapping.RetargetAdditiveTranslationToTargetSkeleton(TargetSkeletonBoneIndex.GetInt(), OutPose[BoneIndex].GetTranslation()));
				}
			}
		}
		else
		{
			for (FCompactPoseBoneIndex BoneIndex(bFirstTrackIsRootBone ? 1 : 0); BoneIndex < NumCompactBones; ++BoneIndex)
			{
				const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = RequiredBones.GetSkeletonPoseIndexFromCompactPoseIndex(BoneIndex);
				if(!TargetSkeletonBoneIndex.IsValid())
				{
					continue;
				}

				OutPose[BoneIndex].SetRotation(SkeletonRemapping.RetargetBoneRotationToTargetSkeleton(TargetSkeletonBoneIndex.GetInt(), OutPose[BoneIndex].GetRotation()));

				// Check what retarget mode to use for the translational retargeting for this specific bone.
				const int32 SourceSkeletonBoneIndex = SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt());
				const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
					bUseSourceRetargetModes,
					SourceSkeletonBoneIndex,
					TargetSkeletonBoneIndex.GetInt(),
					SourceSkeleton,
					TargetSkeleton,
					bDisableRetargeting);

				if (RetargetMode != EBoneTranslationRetargetingMode::Skeleton)
				{
					OutPose[BoneIndex].SetTranslation(SkeletonRemapping.RetargetBoneTranslationToTargetSkeleton(TargetSkeletonBoneIndex.GetInt(), OutPose[BoneIndex].GetTranslation()));
				}
			}
		}
	}

	// Once pose has been extracted, snap root bone back to first frame if we are extracting root motion.
	if ((ExtractionContext.bExtractRootMotion && RootMotionReset.bEnableRootMotion) || RootMotionReset.bForceRootLock)
	{
		RootMotionReset.ResetRootBoneForRootMotion(OutPose[FCompactPoseBoneIndex(0)], RequiredBones);
	}

	// Anim Scale Retargeting
	int32 const NumBonesToScaleRetarget = AnimScaleRetargetingPairs.Num();
	if (NumBonesToScaleRetarget > 0)
	{
		TArray<FTransform> const& AuthoredOnRefSkeleton = RetargetTransforms;

		for (const BoneTrackPair& BonePair : AnimScaleRetargetingPairs)
		{
			const FCompactPoseBoneIndex BoneIndex(BonePair.AtomIndex); //Nasty, we break our type safety, code in the lower levels should be adjusted for this
			const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
			if (!AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
			{
				continue;
			}

			// @todo - precache that in FBoneContainer when we have SkeletonIndex->TrackIndex mapping. So we can just apply scale right away.
			float const SourceTranslationLength = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex].GetTranslation().Size();
			if (SourceTranslationLength > UE_KINDA_SMALL_NUMBER)
			{
				float const TargetTranslationLength = RequiredBones.GetRefPoseTransform(BoneIndex).GetTranslation().Size();
				OutPose[BoneIndex].ScaleTranslation(TargetTranslationLength / SourceTranslationLength);
			}
		}
	}

	// Anim Relative Retargeting
	int32 const NumBonesToRelativeRetarget = AnimRelativeRetargetingPairs.Num();
	if (NumBonesToRelativeRetarget > 0)
	{
		TArray<FTransform> const& AuthoredOnRefSkeleton = RetargetTransforms;

		for (const BoneTrackPair& BonePair : AnimRelativeRetargetingPairs)
		{
			const FCompactPoseBoneIndex BoneIndex(BonePair.AtomIndex); //Nasty, we break our type safety, code in the lower levels should be adjusted for this
			const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
			if (!AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
			{
				continue;
			}

			const FTransform& RefPoseTransform = RequiredBones.GetRefPoseTransform(BoneIndex);

			// Remap the base pose onto the target skeleton so that we are working entirely in target space
			const FTransform& RefBaseTransform = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex];
			const FTransform* BaseTransform = &RefBaseTransform;
			FTransform RetargetBaseTransform;
			if (SkeletonRemapping.RequiresReferencePoseRetarget())
			{
				const int32 TargetSkeletonBoneIndex = SkeletonRemapping.GetTargetSkeletonBoneIndex(SourceSkeletonBoneIndex);
				RetargetBaseTransform = SkeletonRemapping.RetargetBoneTransformToTargetSkeleton(TargetSkeletonBoneIndex, RefBaseTransform);
				BaseTransform = &RetargetBaseTransform;
			}

			// Apply the retargeting as if it were an additive difference between the current skeleton and the retarget skeleton. 
			OutPose[BoneIndex].SetRotation(OutPose[BoneIndex].GetRotation() * BaseTransform->GetRotation().Inverse() * RefPoseTransform.GetRotation());
			OutPose[BoneIndex].SetTranslation(OutPose[BoneIndex].GetTranslation() + (RefPoseTransform.GetTranslation() - BaseTransform->GetTranslation()));
			OutPose[BoneIndex].SetScale3D(OutPose[BoneIndex].GetScale3D() * (RefPoseTransform.GetScale3D() * BaseTransform->GetSafeScaleReciprocal(BaseTransform->GetScale3D())));
			OutPose[BoneIndex].NormalizeRotation();
		}
	}

	// Translation 'Orient and Scale' Translation Retargeting
	const int32 NumBonesToOrientAndScaleRetarget = OrientAndScaleRetargetingPairs.Num();
	if (NumBonesToOrientAndScaleRetarget > 0)
	{
		const FRetargetSourceCachedData& RetargetSourceCachedData = RequiredBones.GetRetargetSourceCachedData(DecompressionContext.AnimName, SkeletonRemapping, RetargetTransforms);
		const TArray<FOrientAndScaleRetargetingCachedData>& OrientAndScaleDataArray = RetargetSourceCachedData.OrientAndScaleData;
		const TArray<int32>& CompactPoseIndexToOrientAndScaleIndex = RetargetSourceCachedData.CompactPoseIndexToOrientAndScaleIndex;

		// If we have any cached retargeting data.
		if (OrientAndScaleDataArray.Num() > 0 && CompactPoseIndexToOrientAndScaleIndex.Num() == NumCompactBones)
		{
			for (int32 Index = 0; Index < NumBonesToOrientAndScaleRetarget; Index++)
			{
				const BoneTrackPair& BonePair = OrientAndScaleRetargetingPairs[Index];
				const FCompactPoseBoneIndex CompactPoseBoneIndex(BonePair.AtomIndex);
				const int32 OrientAndScaleIndex = CompactPoseIndexToOrientAndScaleIndex[CompactPoseBoneIndex.GetInt()];
				if (OrientAndScaleIndex != INDEX_NONE)
				{
					const FOrientAndScaleRetargetingCachedData& OrientAndScaleData = OrientAndScaleDataArray[OrientAndScaleIndex];
					FTransform& BoneTransform = OutPose[CompactPoseBoneIndex];
					const FVector AnimatedTranslation = BoneTransform.GetTranslation();

					// If Translation is not animated, we can just copy the TargetTranslation. No retargeting needs to be done.
					const FVector NewTranslation = (AnimatedTranslation - OrientAndScaleData.SourceTranslation).IsNearlyZero(BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION) ?
						OrientAndScaleData.TargetTranslation :
						OrientAndScaleData.TranslationDeltaOrient.RotateVector(AnimatedTranslation) * OrientAndScaleData.TranslationScale;

					BoneTransform.SetTranslation(NewTranslation);
				}
			}
		}
	}

	if (bIsMeshSpaceAdditive)
	{
		// When an animation is a mesh-space additive, bones that aren't animated will end up with some non-identity
		// delta relative to the base used to create the additive. This is because the delta is calculated in mesh-space
		// unlike regular additive animations where bones that aren't animated has an identity delta. For rotations,
		// this mesh-space delta will be the parent bone rotation.
		// However, if a bone isn't animated in the sequence but present on the target skeleton, we have no data for it
		// and the output pose will contain an identity delta which isn't what we want. As such, bones missing from
		// the sequence have their rotation set to their parent.

		// We always skip the root since it has no parent (its delta value is fine as the identity)
		for (FCompactPoseBoneIndex CompactBoneIndex(1); CompactBoneIndex < NumCompactBones; ++CompactBoneIndex)
		{
			if (!AnimatedCompactRotations[CompactBoneIndex.GetInt()])
			{
				// This bone wasn't animated in the sequence, fix it up
				const FCompactPoseBoneIndex CompactParentIndex = RequiredBones.GetParentBoneIndex(CompactBoneIndex);
				OutPose[CompactBoneIndex].SetRotation(OutPose[CompactParentIndex].GetRotation());
			}
		}
	}
}
}
