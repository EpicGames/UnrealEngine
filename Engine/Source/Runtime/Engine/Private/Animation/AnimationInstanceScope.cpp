// Copyright Epic Games, Inc. All Rights Reserved.


#include "Animation/AnimationInstanceScope.h"

namespace UE::Anim
{
IMPLEMENT_NOTIFY_CONTEXT_INTERFACE(FAnimNotifyAssetPlayerInstanceContext)

FAnimNotifyAssetPlayerInstanceContext::FAnimNotifyAssetPlayerInstanceContext(const uint32 InAssetPlayerInstanceID)
	: AssetPlayerInstanceID(InAssetPlayerInstanceID)
{
}
}
