// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimationPoseData.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimSlotEvaluationPose.h"


FAnimationPoseData::FAnimationPoseData(FPoseContext& InPoseContext)
	: Pose(InPoseContext.Pose), Curve(InPoseContext.Curve), Attributes(InPoseContext.CustomAttributes)
{
}

FAnimationPoseData::FAnimationPoseData(FSlotEvaluationPose& InSlotPoseContext)
	: Pose(InSlotPoseContext.Pose), Curve(InSlotPoseContext.Curve), Attributes(InSlotPoseContext.Attributes)
{
}

FAnimationPoseData::FAnimationPoseData(FCompactPose& InPose, FBlendedCurve& InCurve, UE::Anim::FStackAttributeContainer& InAttributes) : Pose(InPose), Curve(InCurve), Attributes(InAttributes)
{
}

FCompactPose& FAnimationPoseData::GetPose()
{
	return Pose;
}

const FCompactPose& FAnimationPoseData::GetPose() const
{
	return Pose;
}

FBlendedCurve& FAnimationPoseData::GetCurve()
{
	return Curve;
}

const FBlendedCurve& FAnimationPoseData::GetCurve() const
{
	return Curve;
}

UE::Anim::FStackAttributeContainer& FAnimationPoseData::GetAttributes()
{
	return Attributes;
}

const UE::Anim::FStackAttributeContainer& FAnimationPoseData::GetAttributes() const
{
	return Attributes;
}
