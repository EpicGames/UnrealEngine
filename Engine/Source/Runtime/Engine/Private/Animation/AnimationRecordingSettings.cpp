// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimationRecordingSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationRecordingSettings)

/** 30Hz default sample rate */
const FFrameRate FAnimationRecordingSettings::DefaultSampleFrameRate = FFrameRate(30, 1);

/** 1 minute default length */
const float FAnimationRecordingSettings::DefaultMaximumLength = 1.0f * 60.0f;

/** Length used to specify unbounded */
const float FAnimationRecordingSettings::UnboundedMaximumLength = 0.0f;

