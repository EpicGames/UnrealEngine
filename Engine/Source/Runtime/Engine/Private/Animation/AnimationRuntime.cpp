// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationRuntime.cpp: Animation runtime utilities
=============================================================================*/ 

#include "AnimationRuntime.h"
#include "Animation/AnimData/BoneMaskFilter.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimInstance.h"
#include "BonePose.h"
#include "Engine/SkeletalMesh.h"
#include "SkeletalRender.h"
#include "SkeletalRenderPublic.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/BlendProfile.h"
#include "Animation/MirrorDataTable.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/SkeletonRemapping.h"
#include "AutoRTFM.h"

DEFINE_LOG_CATEGORY(LogAnimation);
DEFINE_LOG_CATEGORY(LogRootMotion);

DECLARE_CYCLE_STAT(TEXT("ConvertPoseToMeshRot"), STAT_ConvertPoseToMeshRot, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("ConvertMeshRotPoseToLocalSpace"), STAT_ConvertMeshRotPoseToLocalSpace, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("AccumulateMeshSpaceRotAdditiveToLocalPose"), STAT_AccumulateMeshSpaceRotAdditiveToLocalPose, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("BlendPosesPerBoneFilter"), STAT_BlendPosesPerBoneFilter, STATGROUP_Anim);

//////////////////////////////////////////////////////////////////////////

#if INTEL_ISPC

#include "AnimationRuntime.ispc.generated.h"

static_assert(sizeof(ispc::FTransform) == sizeof(FTransform), "sizeof(ispc::FTransform) != sizeof(FTransform)");
static_assert(sizeof(ispc::FVector) == sizeof(FVector), "sizeof(ispc::FVector) != sizeof(FVector)");
static_assert(sizeof(ispc::FVector4) == sizeof(FQuat), "sizeof(ispc::FVector4) != sizeof(FQuat)");
static_assert(sizeof(ispc::FPerBoneBlendWeight) == sizeof(FPerBoneBlendWeight), "sizeof(ispc::FPerBoneBlendWeight) != sizeof(FPerBoneBlendWeight)");

#if !defined(ANIM_BLEND_POSE_OVERWRITE_ISPC_ENABLED_DEFAULT)
#define ANIM_BLEND_POSE_OVERWRITE_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(ANIM_BLEND_POSE_ACCUMULATE_ISPC_ENABLED_DEFAULT)
#define ANIM_BLEND_POSE_ACCUMULATE_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(ANIM_BLEND_CURVES_ISPC_ENABLED_DEFAULT)
#define ANIM_BLEND_CURVES_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(ANIM_LERP_BONE_TRANSFORMS_ISPC_ENABLED_DEFAULT)
#define ANIM_LERP_BONE_TRANSFORMS_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(ANIM_CONVERT_POSE_TO_MESH_ROTATION_ISPC_ENABLED_DEFAULT)
#define ANIM_CONVERT_POSE_TO_MESH_ROTATION_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(ANIM_CONVERT_MESH_ROTATION_TO_LOCAL_SPACE_ISPC_ENABLED_DEFAULT)
#define ANIM_CONVERT_MESH_ROTATION_TO_LOCAL_SPACE_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(ANIM_ACCUMULATE_LOCAL_SPACE_ADDITIVE_POSE_ISPC_ENABLED_DEFAULT)
#define ANIM_ACCUMULATE_LOCAL_SPACE_ADDITIVE_POSE_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(ANIM_BLEND_POSES_PER_BONE_FILTER_ISPC_ENABLED_DEFAULT)
#define ANIM_BLEND_POSES_PER_BONE_FILTER_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(ANIM_CONVERT_POSE_TO_ADDITIVE_ISPC_ENABLED_DEFAULT)
#define ANIM_CONVERT_POSE_TO_ADDITIVE_ISPC_ENABLED_DEFAULT 1
#endif

#if UE_BUILD_SHIPPING
static constexpr bool bAnim_BlendPoseOverwrite_ISPC_Enabled = ANIM_BLEND_POSE_OVERWRITE_ISPC_ENABLED_DEFAULT;
static constexpr bool bAnim_BlendPoseAccumulate_ISPC_Enabled = ANIM_BLEND_POSE_ACCUMULATE_ISPC_ENABLED_DEFAULT;
static constexpr bool bAnim_BlendCurves_ISPC_Enabled = ANIM_BLEND_CURVES_ISPC_ENABLED_DEFAULT;
static constexpr bool bAnim_LerpBoneTransforms_ISPC_Enabled = ANIM_LERP_BONE_TRANSFORMS_ISPC_ENABLED_DEFAULT;
static constexpr bool bAnim_ConvertPoseToMeshRotation_ISPC_Enabled = ANIM_CONVERT_POSE_TO_MESH_ROTATION_ISPC_ENABLED_DEFAULT;
static constexpr bool bAnim_ConvertMeshRotationPoseToLocalSpace_ISPC_Enabled = ANIM_CONVERT_MESH_ROTATION_TO_LOCAL_SPACE_ISPC_ENABLED_DEFAULT;
static constexpr bool bAnim_AccumulateLocalSpaceAdditivePose_ISPC_Enabled = ANIM_ACCUMULATE_LOCAL_SPACE_ADDITIVE_POSE_ISPC_ENABLED_DEFAULT;
static constexpr bool bAnim_BlendPosesPerBoneFilter_ISPC_Enabled = ANIM_BLEND_POSES_PER_BONE_FILTER_ISPC_ENABLED_DEFAULT;
static constexpr bool bAnim_ConvertPoseToAdditive_ISPC_Enabled = ANIM_CONVERT_POSE_TO_ADDITIVE_ISPC_ENABLED_DEFAULT;
#else
static bool bAnim_BlendPoseOverwrite_ISPC_Enabled = ANIM_BLEND_POSE_OVERWRITE_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarBlendPoseOverwriteISPCEnabled(TEXT("a.BlendPoseOverwrite.ISPC"), bAnim_BlendPoseOverwrite_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for over-write pose blending"));
static bool bAnim_BlendPoseAccumulate_ISPC_Enabled = ANIM_BLEND_POSE_ACCUMULATE_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarBlendPoseAccumulateISPCEnabled(TEXT("a.BlendPoseAccumulate.ISPC"), bAnim_BlendPoseAccumulate_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for accumulation pose blending"));
static bool bAnim_BlendCurves_ISPC_Enabled = ANIM_BLEND_CURVES_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarBlendCurvesISPCEnabled(TEXT("a.BlendCurves.ISPC"), bAnim_BlendCurves_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for curve blending"));
static bool bAnim_LerpBoneTransforms_ISPC_Enabled = ANIM_LERP_BONE_TRANSFORMS_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarLerpBoneTransformsISPCEnabled(TEXT("a.LerpBoneTransforms.ISPC"), bAnim_LerpBoneTransforms_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for interpolating bone transforms"));
static bool bAnim_ConvertPoseToMeshRotation_ISPC_Enabled = ANIM_CONVERT_POSE_TO_MESH_ROTATION_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarConvertPoseToMeshRotationISPCEnabled(TEXT("a.ConvertPoseToMeshRotation.ISPC"), bAnim_ConvertPoseToMeshRotation_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for converting local space rotations to mesh space"));
static bool bAnim_ConvertMeshRotationPoseToLocalSpace_ISPC_Enabled = ANIM_CONVERT_MESH_ROTATION_TO_LOCAL_SPACE_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarConvertMeshRotationPoseToLocalSpace(TEXT("a.ConvertMeshRotationPoseToLocalSpace.ISPC"), bAnim_ConvertMeshRotationPoseToLocalSpace_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for converting mesh space rotations to local space"));
static bool bAnim_AccumulateLocalSpaceAdditivePose_ISPC_Enabled = ANIM_ACCUMULATE_LOCAL_SPACE_ADDITIVE_POSE_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarAccumulateLocalSpaceAdditivePose(TEXT("a.AccumulateLocalSpaceAdditivePose.ISPC"), bAnim_AccumulateLocalSpaceAdditivePose_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for accumulating local space additive pose"));
static bool bAnim_BlendPosesPerBoneFilter_ISPC_Enabled = ANIM_BLEND_POSES_PER_BONE_FILTER_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarBlendPosesPerBoneFilter(TEXT("a.BlendPosesPerBoneFilter.ISPC"), bAnim_BlendPosesPerBoneFilter_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for blending poses with a per-bone filter"));
static bool bAnim_ConvertPoseToAdditive_ISPC_Enabled = ANIM_CONVERT_POSE_TO_ADDITIVE_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarConvertPoseToAdditiveISPCEnabled(TEXT("a.ConvertPoseToAdditive.ISPC"), bAnim_ConvertPoseToAdditive_ISPC_Enabled, TEXT("Whether to use ISPC optimizations for converting poses to additive poses"));
#endif // UE_BUILD_SHIPPING

#endif // INTEL_ISPC

void FAnimationRuntime::NormalizeRotations(const FBoneContainer& RequiredBones, /*inout*/ FTransformArrayA2& Atoms)
{
	check( Atoms.Num() == RequiredBones.GetNumBones() );
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	for (int32 j = 0; j < RequiredBoneIndices.Num(); ++j)
	{
		const int32 BoneIndex = RequiredBoneIndices[j];
		Atoms[BoneIndex].NormalizeRotation();
	}
}

void FAnimationRuntime::NormalizeRotations(FTransformArrayA2& Atoms)
{
	for (int32 BoneIndex = 0; BoneIndex < Atoms.Num(); BoneIndex++)
	{
		Atoms[BoneIndex].NormalizeRotation();
	}
}

void FAnimationRuntime::InitializeTransform(const FBoneContainer& RequiredBones, /*inout*/ FTransformArrayA2& Atoms)
{
	check( Atoms.Num() == RequiredBones.GetNumBones() );
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	for (int32 j = 0; j < RequiredBoneIndices.Num(); ++j)
	{
		const int32 BoneIndex = RequiredBoneIndices[j];
		Atoms[BoneIndex].SetIdentity();
	}
}

template <int32 TRANSFORM_BLEND_MODE>
FORCEINLINE void BlendPose(const FTransformArrayA2& SourcePoses, FTransformArrayA2& ResultAtoms, const TArray<FBoneIndexType>& RequiredBoneIndices, const float BlendWeight)
{
	for (int32 i = 0; i < RequiredBoneIndices.Num(); ++i)
	{
		const int32 BoneIndex = RequiredBoneIndices[i];
		BlendTransform<TRANSFORM_BLEND_MODE>(SourcePoses[BoneIndex], ResultAtoms[BoneIndex], BlendWeight);
	}
}

template <int32 TRANSFORM_BLEND_MODE>
FORCEINLINE void BlendPose(const FCompactPose& SourcePose, FCompactPose& ResultPose, const float BlendWeight)
{
	for (FCompactPoseBoneIndex BoneIndex : SourcePose.ForEachBoneIndex())
	{
		BlendTransform<TRANSFORM_BLEND_MODE>(SourcePose[BoneIndex], ResultPose[BoneIndex], BlendWeight);
	}
}

template <>
FORCEINLINE void BlendPose<ETransformBlendMode::Overwrite>(const FCompactPose& SourcePose, FCompactPose& ResultPose, const float BlendWeight)
{
#if INTEL_ISPC
	if (bAnim_BlendPoseOverwrite_ISPC_Enabled)
	{
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				ResultPose.GetMutableBones().GetData(),
				ResultPose.GetNumBones() * sizeof(FTransform));
			ispc::BlendTransformOverwrite(
				reinterpret_cast<const ispc::FTransform*>(SourcePose.GetBones().GetData()),
				reinterpret_cast<ispc::FTransform*>(ResultPose.GetMutableBones().GetData()),
				BlendWeight,
				SourcePose.GetNumBones());
		};
	}
	else
#endif
	{
		for (FCompactPoseBoneIndex BoneIndex : SourcePose.ForEachBoneIndex())
		{
			BlendTransform<ETransformBlendMode::Overwrite>(SourcePose[BoneIndex], ResultPose[BoneIndex], BlendWeight);
		}
	}
}

template <>
FORCEINLINE void BlendPose<ETransformBlendMode::Accumulate>(const FCompactPose& SourcePose, FCompactPose& ResultPose, const float BlendWeight)
{
#if INTEL_ISPC
	if (bAnim_BlendPoseAccumulate_ISPC_Enabled)
	{
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				ResultPose.GetMutableBones().GetData(),
				ResultPose.GetNumBones() * sizeof(FTransform));
			ispc::BlendTransformAccumulate(
				reinterpret_cast<const ispc::FTransform*>(SourcePose.GetBones().GetData()),
				reinterpret_cast<ispc::FTransform*>(ResultPose.GetMutableBones().GetData()),
				BlendWeight,
				SourcePose.GetNumBones());
		};
	}
	else
#endif
	{
		for (FCompactPoseBoneIndex BoneIndex : SourcePose.ForEachBoneIndex())
		{
			BlendTransform<ETransformBlendMode::Accumulate>(SourcePose[BoneIndex], ResultPose[BoneIndex], BlendWeight);
		}
	}
}

FORCEINLINE void BlendCurves(const TArrayView<const FBlendedCurve> SourceCurves, const TArrayView<const float> SourceWeights, const TArrayView<const int32> SourceWeightsIndices, FBlendedCurve& OutCurve)
{
	if (SourceCurves.Num() > 0)
	{
		OutCurve.Override(SourceCurves[0], SourceWeights[SourceWeightsIndices[0]]);

		for (int32 CurveIndex = 1; CurveIndex<SourceCurves.Num(); ++CurveIndex)
		{
			OutCurve.Accumulate(SourceCurves[CurveIndex], SourceWeights[SourceWeightsIndices[CurveIndex]]);
		}
	}
}

void BlendCurves(const TArrayView<const FBlendedCurve* const> SourceCurves, const TArrayView<const float> SourceWeights, FBlendedCurve& OutCurve)
{
	if(SourceCurves.Num() > 0)
	{
		OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

		for(int32 CurveIndex=1; CurveIndex<SourceCurves.Num(); ++CurveIndex)
		{
			OutCurve.Accumulate(*SourceCurves[CurveIndex], SourceWeights[CurveIndex]);
		}
	}
}

FORCEINLINE void BlendCurves(const TArrayView<const FBlendedCurve> SourceCurves, const TArrayView<const float> SourceWeights, FBlendedCurve& OutCurve)
{
	if (SourceCurves.Num() > 0)
	{
/*#if INTEL_ISPC
		if (bAnim_BlendCurves_ISPC_Enabled)
		{
			OutCurve.InitFrom(SourceCurves[0]);
			for (int32 CurveIndex = 0; CurveIndex < SourceCurves.Num(); ++CurveIndex)
			{
				ispc::BlendCurves(
					SourceCurves[CurveIndex].CurveWeights.GetData(),
					SourceCurves[CurveIndex].ValidCurveWeights.GetData(),
					OutCurve.CurveWeights.GetData(),
					OutCurve.ValidCurveWeights.GetData(),
					OutCurve.CurveWeights.Num(),
					CurveIndex,
					SourceWeights[CurveIndex]
				);
			}
		}
		else
#endif*/
		{
			OutCurve.Override(SourceCurves[0], SourceWeights[0]);
			for (int32 CurveIndex = 1; CurveIndex < SourceCurves.Num(); ++CurveIndex)
			{
				OutCurve.Accumulate(SourceCurves[CurveIndex], SourceWeights[CurveIndex]);
			}
		}
	}
}

void BlendCurves(const TArrayView<const FBlendedCurve* const> SourceCurves, const TArrayView<const float> SourceWeights, FBlendedCurve& OutCurve, ECurveBlendOption::Type BlendOption)
{
	if(SourceCurves.Num() > 0)
	{
		if (BlendOption == ECurveBlendOption::Type::BlendByWeight)
		{
			BlendCurves(SourceCurves, SourceWeights, OutCurve);
		}
		else if (BlendOption == ECurveBlendOption::Type::NormalizeByWeight)
		{
			float SumOfWeight = 0.f;
			for (const auto& Weight : SourceWeights)
			{
				SumOfWeight += Weight;
			}

			if (FAnimWeight::IsRelevant(SumOfWeight))
			{
				TArray<float> NormalizeSourceWeights;
				NormalizeSourceWeights.AddUninitialized(SourceWeights.Num());
				for(int32 Idx=0; Idx<SourceWeights.Num(); ++Idx)
				{
					NormalizeSourceWeights[Idx] = SourceWeights[Idx] / SumOfWeight;
				}

				BlendCurves(SourceCurves, NormalizeSourceWeights, OutCurve);
			}
			else
			{
				BlendCurves(SourceCurves, SourceWeights, OutCurve);
			}
		}
		else if (BlendOption == ECurveBlendOption::Type::UseMaxValue)
		{
			OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

			for (int32 CurveIndex = 1; CurveIndex < SourceCurves.Num(); ++CurveIndex)
			{
				OutCurve.UseMaxValue(*SourceCurves[CurveIndex]);
			}
		}
		else if (BlendOption == ECurveBlendOption::Type::UseMinValue)
		{
			OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

			for (int32 CurveIndex = 1; CurveIndex < SourceCurves.Num(); ++CurveIndex)
			{
				OutCurve.UseMinValue(*SourceCurves[CurveIndex]);
			}
		}
		else if (BlendOption == ECurveBlendOption::Type::UseBasePose)
		{
			OutCurve.Override(*SourceCurves[0], SourceWeights[0]);
		}
		else if (BlendOption == ECurveBlendOption::Type::DoNotOverride)
		{
			OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

			for (int32 CurveIndex = 1; CurveIndex < SourceCurves.Num(); ++CurveIndex)
			{
				OutCurve.CombinePreserved(*SourceCurves[CurveIndex]);
			}
		}
		else
		{
			OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

			for(int32 CurveIndex=1; CurveIndex<SourceCurves.Num(); ++CurveIndex)
			{
				OutCurve.Combine(*SourceCurves[CurveIndex]);
			}
		}
	}
}

void FAnimationRuntime::BlendPosesTogether(
	const TArrayView<const FCompactPose> SourcePoses,
	const TArrayView<const FBlendedCurve> SourceCurves,
	const TArrayView<const float> SourceWeights,
	/*out*/ FCompactPose& ResultPose, 
	/*out*/ FBlendedCurve& ResultCurve)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData = { ResultPose, ResultCurve, TempAttributes };

	BlendPosesTogether(SourcePoses, SourceCurves, {}, SourceWeights, AnimationPoseData);	
}

void FAnimationRuntime::BlendPosesTogether(TArrayView<const FCompactPose> SourcePoses, TArrayView<const FBlendedCurve> SourceCurves, TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes, TArrayView<const float> SourceWeights, FAnimationPoseData& OutAnimationPoseData)
{
	check(SourcePoses.Num() > 0);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	BlendPose<ETransformBlendMode::Overwrite>(SourcePoses[0], OutPose, SourceWeights[0]);

	for (int32 PoseIndex = 1; PoseIndex < SourcePoses.Num(); ++PoseIndex)
	{
		BlendPose<ETransformBlendMode::Accumulate>(SourcePoses[PoseIndex], OutPose, SourceWeights[PoseIndex]);
	}

	// Ensure that all of the resulting rotations are normalized
	if (SourcePoses.Num() > 1)
	{
		OutPose.NormalizeRotations();
	}

	// curve blending if exists
	if (SourceCurves.Num() > 0)
	{
		BlendCurves(SourceCurves, SourceWeights, OutCurve);
	}

	if (SourceAttributes.Num() > 0)
	{
		UE::Anim::Attributes::BlendAttributes(SourceAttributes, SourceWeights, OutAttributes);
	}
}

void FAnimationRuntime::BlendPosesTogether(
	const TArrayView<const FCompactPose> SourcePoses,
	const TArrayView<const FBlendedCurve> SourceCurves,
	const TArrayView<const float> SourceWeights,
	const TArrayView<const int32> SourceWeightsIndices,
	/*out*/ FCompactPose& ResultPose,
	/*out*/ FBlendedCurve& ResultCurve)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData = { ResultPose, ResultCurve, TempAttributes };

	BlendPosesTogether(SourcePoses, SourceCurves, {}, SourceWeights, SourceWeightsIndices, AnimationPoseData);
}


void FAnimationRuntime::BlendPosesTogether(TArrayView<const FCompactPose> SourcePoses, TArrayView<const FBlendedCurve> SourceCurves, TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes, TArrayView<const float> SourceWeights, TArrayView<const int32> SourceWeightsIndices, /*out*/ FAnimationPoseData& OutAnimationPoseData)
{
	check(SourcePoses.Num() > 0);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	BlendPose<ETransformBlendMode::Overwrite>(SourcePoses[0], OutPose, SourceWeights[SourceWeightsIndices[0]]);

	for (int32 PoseIndex = 1; PoseIndex < SourcePoses.Num(); ++PoseIndex)
	{
		BlendPose<ETransformBlendMode::Accumulate>(SourcePoses[PoseIndex], OutPose, SourceWeights[SourceWeightsIndices[PoseIndex]]);
	}

	// Ensure that all of the resulting rotations are normalized
	if (SourcePoses.Num() > 1)
	{
		OutPose.NormalizeRotations();
	}

	// curve blending if exists
	if (SourceCurves.Num() > 0)
	{
		BlendCurves(SourceCurves, SourceWeights, SourceWeightsIndices, OutCurve);
	}

	if (SourceAttributes.Num() > 0)
	{
		UE::Anim::Attributes::BlendAttributes(SourceAttributes, SourceWeights, SourceWeightsIndices, OutAttributes);
	}
}

void FAnimationRuntime::BlendPosesTogetherIndirect(
	const TArrayView<const FCompactPose* const> SourcePoses,
	const TArrayView<const FBlendedCurve* const> SourceCurves,
	const TArrayView<const float> SourceWeights,
	/*out*/ FCompactPose& ResultPose,
	/*out*/ FBlendedCurve& ResultCurve)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData = { ResultPose, ResultCurve, TempAttributes };

	BlendPosesTogetherIndirect(SourcePoses, SourceCurves, {}, SourceWeights, AnimationPoseData);
}

void FAnimationRuntime::BlendPosesTogetherIndirect(TArrayView<const FCompactPose* const> SourcePoses, TArrayView<const FBlendedCurve* const> SourceCurves, TArrayView<const UE::Anim::FStackAttributeContainer* const> SourceAttributes, TArrayView<const float> SourceWeights, FAnimationPoseData& OutAnimationPoseData)
{
	check(SourcePoses.Num() > 0);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();
	
	BlendPose<ETransformBlendMode::Overwrite>(*SourcePoses[0], OutPose, SourceWeights[0]);

	for (int32 PoseIndex = 1; PoseIndex < SourcePoses.Num(); ++PoseIndex)
	{
		BlendPose<ETransformBlendMode::Accumulate>(*SourcePoses[PoseIndex], OutPose, SourceWeights[PoseIndex]);
	}

	// Ensure that all of the resulting rotations are normalized
	if (SourcePoses.Num() > 1)
	{
		OutPose.NormalizeRotations();
	}

	if (SourceCurves.Num() > 0)
	{
		BlendCurves(SourceCurves, SourceWeights, OutCurve);
	}

	if (SourceAttributes.Num() > 0)
	{
		UE::Anim::Attributes::BlendAttributes(SourceAttributes, SourceWeights, OutAttributes);
	}
}

void FAnimationRuntime::BlendTwoPosesTogether(
	const FCompactPose& SourcePose1,
	const FCompactPose& SourcePose2,
	const FBlendedCurve& SourceCurve1,
	const FBlendedCurve& SourceCurve2,
	const float			WeightOfPose1,
	/*out*/ FCompactPose& ResultPose,
	/*out*/ FBlendedCurve& ResultCurve)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	
	FAnimationPoseData AnimationPoseData = { ResultPose, ResultCurve, TempAttributes };
	
	const FAnimationPoseData SourceOnePoseData(const_cast<FCompactPose&>(SourcePose1), const_cast<FBlendedCurve&>(SourceCurve1), TempAttributes);
	const FAnimationPoseData SourceTwoPosedata(const_cast<FCompactPose&>(SourcePose2), const_cast<FBlendedCurve&>(SourceCurve2), TempAttributes);

	BlendTwoPosesTogether(SourceOnePoseData, SourceTwoPosedata, WeightOfPose1, AnimationPoseData);
}

void FAnimationRuntime::BlendTwoPosesTogether(const FAnimationPoseData& SourcePoseOneData, const FAnimationPoseData& SourcePoseTwoData, const float WeightOfPoseOne, /*out*/ FAnimationPoseData& OutAnimationPoseData)
{
	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	const float WeightOfPoseTwo = 1.f - WeightOfPoseOne;

	BlendPose<ETransformBlendMode::Overwrite>(SourcePoseOneData.GetPose(), OutPose, WeightOfPoseOne);
	BlendPose<ETransformBlendMode::Accumulate>(SourcePoseTwoData.GetPose(), OutPose, WeightOfPoseTwo);

	// Ensure that all of the resulting rotations are normalized
	OutPose.NormalizeRotations();

	OutCurve.Lerp(SourcePoseOneData.GetCurve(), SourcePoseTwoData.GetCurve(), WeightOfPoseTwo);
	UE::Anim::Attributes::BlendAttributes({ SourcePoseOneData.GetAttributes(), SourcePoseTwoData.GetAttributes() }, { WeightOfPoseOne, WeightOfPoseTwo }, { 0, 1 }, OutAttributes);
}

void FAnimationRuntime::BlendTwoPosesTogetherInPlace(FAnimationPoseData& InOutPoseOneData, const FAnimationPoseData& SourcePoseTwoData, const float WeightOfPoseOne)
{
	FCompactPose& InOutPose = InOutPoseOneData.GetPose();
	FBlendedCurve& InOutCurve = InOutPoseOneData.GetCurve();
	UE::Anim::FStackAttributeContainer& InOutAttributes = InOutPoseOneData.GetAttributes();

	const float WeightOfPoseTwo = 1.f - WeightOfPoseOne;

	// Scale first pose in-place, then accumulate second pose
	BlendPose<ETransformBlendMode::Overwrite>(InOutPose, InOutPose, WeightOfPoseOne);
	BlendPose<ETransformBlendMode::Accumulate>(SourcePoseTwoData.GetPose(), InOutPose, WeightOfPoseTwo);

	// Ensure that all of the resulting rotations are normalized
	InOutPose.NormalizeRotations();

	// Use 3 args Lerp over 2 args in-place LerpTo as this is more performant and the memory overhead is negligible
	FBlendedCurve TempCurve;
	TempCurve.Lerp(InOutCurve, SourcePoseTwoData.GetCurve(), WeightOfPoseTwo);
	InOutCurve = MoveTemp(TempCurve);

	// Use a temporary for attributes to avoid aliasing
	UE::Anim::FStackAttributeContainer TempAttributes;
	UE::Anim::Attributes::BlendAttributes({ InOutPoseOneData.GetAttributes(), SourcePoseTwoData.GetAttributes() }, { WeightOfPoseOne, WeightOfPoseTwo }, { 0, 1 }, TempAttributes);
	InOutAttributes = MoveTemp(TempAttributes);
}

void FAnimationRuntime::BlendTwoPosesTogetherPerBone(
		const FCompactPose& SourcePose,
		const FCompactPose& TargetPose,
		const TArray<float>& TargetWeights,
		/*out*/ FCompactPose& OutPose)
{
	for (FCompactPoseBoneIndex BoneIndex : OutPose.ForEachBoneIndex())
	{
		const float BlendWeight = TargetWeights[BoneIndex.GetInt()];
		if (FAnimationRuntime::IsFullWeight(BlendWeight))
		{
			OutPose[BoneIndex] = TargetPose[BoneIndex];
		}
		// if it doesn't have weight, take source pose 1
		else if (FAnimationRuntime::HasWeight(BlendWeight))
		{
			BlendTransform<ETransformBlendMode::Overwrite>(SourcePose[BoneIndex], OutPose[BoneIndex], 1.f - BlendWeight);
			BlendTransform<ETransformBlendMode::Accumulate>(TargetPose[BoneIndex], OutPose[BoneIndex], BlendWeight);
		}
		else
		{
			OutPose[BoneIndex] = SourcePose[BoneIndex];
		}
	}

	// Ensure that all of the resulting rotations are normalized
	OutPose.NormalizeRotations();
}

void FAnimationRuntime::BlendTwoPosesTogetherPerBone(
	const FCompactPose& SourcePose1,
	const FCompactPose& SourcePose2,
	const FBlendedCurve& SourceCurve1,
	const FBlendedCurve& SourceCurve2,
	const TArray<float>& WeightsOfSource2,
	/*out*/ FCompactPose& ResultPose,
	/*out*/ FBlendedCurve& ResultCurve)
{
	UE::Anim::FStackAttributeContainer TempAttributes;

	FAnimationPoseData AnimationPoseData = { ResultPose, ResultCurve, TempAttributes };
	const FAnimationPoseData SourceOnePoseData(const_cast<FCompactPose&>(SourcePose1), const_cast<FBlendedCurve&>(SourceCurve1), TempAttributes);
	const FAnimationPoseData SourceTwoPosedata(const_cast<FCompactPose&>(SourcePose2), const_cast<FBlendedCurve&>(SourceCurve2), TempAttributes);

	BlendTwoPosesTogetherPerBone(SourceOnePoseData, SourceTwoPosedata, WeightsOfSource2, AnimationPoseData);
}

void FAnimationRuntime::BlendTwoPosesTogetherPerBone(const FAnimationPoseData& SourcePoseOneData, const FAnimationPoseData& SourcePoseTwoData, const TArray<float>& WeightsOfSource2, /*out*/ FAnimationPoseData& OutAnimationPoseData)
{
	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	const FCompactPose& SourcePoseOne = SourcePoseOneData.GetPose();
	const FCompactPose& SourcePoseTwo = SourcePoseTwoData.GetPose();
	BlendTwoPosesTogetherPerBone(SourcePoseOne, SourcePoseTwo, WeightsOfSource2, OutPose);

	// @note : This isn't perfect as curve can link to joint, and it would be the best to use that information
	// but that is very expensive option as we have to have another indirect look up table to search. 
	// For now, replacing with combine (non-zero will be overriden)
	// in the future, we might want to do this outside if we want per bone blend to apply curve also UE-39182

	const FBlendedCurve& SourceCurveOne = SourcePoseOneData.GetCurve();
	const FBlendedCurve& SourceCurveTwo = SourcePoseTwoData.GetCurve();
	OutCurve.Override(SourceCurveOne);
	OutCurve.Combine(SourceCurveTwo);

	UE::Anim::Attributes::BlendAttributesPerBone(SourcePoseOneData.GetAttributes(), SourcePoseTwoData.GetAttributes(), WeightsOfSource2, OutAttributes);
}

template <int32 TRANSFORM_BLEND_MODE>
void BlendPosePerBone(const TArrayView<int32> RequiredBoneIndices, const TArrayView<int32> PerBoneIndices, const FBlendSampleData& BlendSampleDataCache, FTransformArrayA2& ResultAtoms, const FTransformArrayA2& SourceAtoms)
{
	const float BlendWeight = BlendSampleDataCache.GetClampedWeight();
	for (int32 i = 0; i < RequiredBoneIndices.Num(); ++i)
	{
		const int32 BoneIndex = RequiredBoneIndices[i];
		const int32 PerBoneIndex = PerBoneIndices[i];
		if (PerBoneIndex == INDEX_NONE || !BlendSampleDataCache.PerBoneBlendData.IsValidIndex(PerBoneIndex))
		{
			BlendTransform<TRANSFORM_BLEND_MODE>(SourceAtoms[BoneIndex], ResultAtoms[BoneIndex], BlendWeight);
		}
		else
		{
			BlendTransform<TRANSFORM_BLEND_MODE>(SourceAtoms[BoneIndex], ResultAtoms[BoneIndex], BlendSampleDataCache.PerBoneBlendData[PerBoneIndex]);
		}
	}
}

template <int32 TRANSFORM_BLEND_MODE>
void BlendPosePerBone(const TArrayView<int32> PerBoneIndices, const FBlendSampleData& BlendSampleDataCache, FCompactPose& ResultPose, const FCompactPose& SourcePose)
{
	const float BlendWeight = BlendSampleDataCache.GetClampedWeight();
	for (FCompactPoseBoneIndex BoneIndex : SourcePose.ForEachBoneIndex())
	{
		const int32 PerBoneIndex = PerBoneIndices[BoneIndex.GetInt()];
		if (PerBoneIndex == INDEX_NONE || !BlendSampleDataCache.PerBoneBlendData.IsValidIndex(PerBoneIndex))
		{
			BlendTransform<TRANSFORM_BLEND_MODE>(SourcePose[BoneIndex], ResultPose[BoneIndex], BlendWeight);
		}
		else
		{
			BlendTransform<TRANSFORM_BLEND_MODE>(SourcePose[BoneIndex], ResultPose[BoneIndex], BlendSampleDataCache.PerBoneBlendData[PerBoneIndex]);
		}
	}
}

void FAnimationRuntime::BlendPosesTogetherPerBone(
	const TArrayView<const FCompactPose> SourcePoses,
	const TArrayView<const FBlendedCurve> SourceCurves,
	const IInterpolationIndexProvider* InterpolationIndexProvider,
	const TArrayView<const FBlendSampleData> BlendSampleDataCache,
	/*out*/ FCompactPose& ResultPose,
	/*out*/ FBlendedCurve& ResultCurve)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData = { ResultPose, ResultCurve, TempAttributes };

	BlendPosesTogetherPerBone(SourcePoses, SourceCurves, {}, InterpolationIndexProvider, BlendSampleDataCache, AnimationPoseData);	
}

namespace PerBoneScratch
{
	static thread_local TArray<int32> PerBoneIndices;
	static thread_local TArray<float> SourceWeights;
}

void FAnimationRuntime::BlendPosesTogetherPerBone(TArrayView<const FCompactPose> SourcePoses, TArrayView<const FBlendedCurve> SourceCurves, TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes, const IInterpolationIndexProvider* InterpolationIndexProvider, TArrayView<const FBlendSampleData> BlendSampleDataCache, /*out*/ FAnimationPoseData& OutAnimationPoseData)
{
	check(SourcePoses.Num() > 0);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	TArray<int32>& PerBoneIndices = PerBoneScratch::PerBoneIndices;
	PerBoneIndices.SetNumUninitialized(OutPose.GetNumBones(), EAllowShrinking::No);

	const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();
	TSharedPtr<IInterpolationIndexProvider::FPerBoneInterpolationData> Data = InterpolationIndexProvider->GetPerBoneInterpolationData(RequiredBones.GetSkeletonAsset());
	for(FCompactPoseBoneIndex BoneIndex : OutPose.ForEachBoneIndex())
	{
		PerBoneIndices[BoneIndex.GetInt()] = InterpolationIndexProvider->GetPerBoneInterpolationIndex(BoneIndex, RequiredBones, Data.Get());
	}

	BlendPosePerBone<ETransformBlendMode::Overwrite>(PerBoneIndices, BlendSampleDataCache[0], OutPose, SourcePoses[0]);
	for (int32 i = 1; i < SourcePoses.Num(); ++i)
	{
		BlendPosePerBone<ETransformBlendMode::Accumulate>(PerBoneIndices, BlendSampleDataCache[i], OutPose, SourcePoses[i]);
	}

	// Ensure that all of the resulting rotations are normalized
	OutPose.NormalizeRotations();

	if (SourceCurves.Num() > 0)
	{
		TArray<float>& SourceWeights = PerBoneScratch::SourceWeights;
		SourceWeights.SetNumUninitialized(BlendSampleDataCache.Num(), EAllowShrinking::No);
		for (int32 CacheIndex = 0; CacheIndex < BlendSampleDataCache.Num(); ++CacheIndex)
		{
			SourceWeights[CacheIndex] = BlendSampleDataCache[CacheIndex].TotalWeight;
		}

		BlendCurves(SourceCurves, SourceWeights, OutCurve);
	}
	
	if (SourceAttributes.Num() > 0)
	{
		UE::Anim::Attributes::BlendAttributesPerBone(SourceAttributes, PerBoneIndices, BlendSampleDataCache, OutAttributes);
	}
}

void FAnimationRuntime::BlendPosesTogetherPerBone(
	const TArrayView<const FCompactPose> SourcePoses,
	const TArrayView<const FBlendedCurve> SourceCurves,
	const IInterpolationIndexProvider* InterpolationIndexProvider,
	const TArrayView<const FBlendSampleData> BlendSampleDataCache,
	const TArrayView<const int32> BlendSampleDataCacheIndices,
	/*out*/ FCompactPose& ResultPose,
	/*out*/ FBlendedCurve& ResultCurve)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData = { ResultPose, ResultCurve, TempAttributes };

	BlendPosesTogetherPerBone(SourcePoses, SourceCurves, {}, InterpolationIndexProvider, BlendSampleDataCache, BlendSampleDataCacheIndices, AnimationPoseData);
}

void FAnimationRuntime::BlendPosesTogetherPerBone(TArrayView<const FCompactPose> SourcePoses, TArrayView<const FBlendedCurve> SourceCurves, TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes, const IInterpolationIndexProvider* InterpolationIndexProvider, TArrayView<const FBlendSampleData> BlendSampleDataCache, TArrayView<const int32> BlendSampleDataCacheIndices, /*out*/ FAnimationPoseData& OutAnimationPoseData)
{
	check(SourcePoses.Num() > 0);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	TArray<int32>& PerBoneIndices = PerBoneScratch::PerBoneIndices;
	PerBoneIndices.SetNumUninitialized(OutPose.GetNumBones(), EAllowShrinking::No);

	const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();

	TSharedPtr<IInterpolationIndexProvider::FPerBoneInterpolationData> Data = InterpolationIndexProvider->GetPerBoneInterpolationData(OutPose.GetBoneContainer().GetSkeletonAsset());
	for(FCompactPoseBoneIndex BoneIndex : OutPose.ForEachBoneIndex())
	{
		PerBoneIndices[BoneIndex.GetInt()] = InterpolationIndexProvider->GetPerBoneInterpolationIndex(BoneIndex, RequiredBones, Data.Get());
	}

	BlendPosePerBone<ETransformBlendMode::Overwrite>(PerBoneIndices, BlendSampleDataCache[BlendSampleDataCacheIndices[0]], OutPose, SourcePoses[0]);

	for (int32 i = 1; i < SourcePoses.Num(); ++i)
	{
		BlendPosePerBone<ETransformBlendMode::Accumulate>(PerBoneIndices, BlendSampleDataCache[BlendSampleDataCacheIndices[i]], OutPose, SourcePoses[i]);
	}

	// Ensure that all of the resulting rotations are normalized
	OutPose.NormalizeRotations();

	if (SourceCurves.Num() > 0)
	{
		TArray<float>& SourceWeights = PerBoneScratch::SourceWeights;
		SourceWeights.SetNumUninitialized(BlendSampleDataCacheIndices.Num(), EAllowShrinking::No);
		for (int32 CacheIndex = 0; CacheIndex < BlendSampleDataCacheIndices.Num(); ++CacheIndex)
		{
			SourceWeights[CacheIndex] = BlendSampleDataCache[BlendSampleDataCacheIndices[CacheIndex]].TotalWeight;
		}

		BlendCurves(SourceCurves, SourceWeights, OutCurve);		
	}	

	if (SourceAttributes.Num() > 0)
	{
		UE::Anim::Attributes::BlendAttributesPerBone(SourceAttributes, PerBoneIndices, BlendSampleDataCache, BlendSampleDataCacheIndices, OutAttributes);
	}
}

void FAnimationRuntime::BlendPosesTogetherPerBoneRemapped(TArrayView<const FCompactPose> SourcePoses, TArrayView<const FBlendedCurve> SourceCurves, TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes, const IInterpolationIndexProvider* InterpolationIndexProvider, TArrayView<const FBlendSampleData> BlendSampleDataCache, TArrayView<const int32> BlendSampleDataCacheIndices, const FSkeletonRemapping& SkeletonRemapping, /*out*/ FAnimationPoseData& OutAnimationPoseData)
{
	check(SourcePoses.Num() > 0);
	check(SkeletonRemapping.IsValid());	// If this fails, you most likely want to use BlendPosesTogetherPerBone instead.

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();
	TArray<int32>& PerBoneIndices = PerBoneScratch::PerBoneIndices;
	PerBoneIndices.SetNumUninitialized(OutPose.GetNumBones(), EAllowShrinking::No);
	TSharedPtr<IInterpolationIndexProvider::FPerBoneInterpolationData> Data = InterpolationIndexProvider->GetPerBoneInterpolationData(OutPose.GetBoneContainer().GetSkeletonAsset());
	for (FCompactPoseBoneIndex BoneIndex : OutPose.ForEachBoneIndex())
	{
		const FSkeletonPoseBoneIndex SourceSkelBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(BoneIndex.GetInt()));
		const FCompactPoseBoneIndex SourceBoneIndex = FCompactPoseBoneIndex(RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(SourceSkelBoneIndex));
		PerBoneIndices[BoneIndex.GetInt()] = (SourceBoneIndex != INDEX_NONE) ? InterpolationIndexProvider->GetPerBoneInterpolationIndex(SourceBoneIndex, RequiredBones, Data.Get()) : INDEX_NONE;
	}

	BlendPosePerBone<ETransformBlendMode::Overwrite>(PerBoneIndices, BlendSampleDataCache[BlendSampleDataCacheIndices[0]], OutPose, SourcePoses[0]);

	for (int32 i = 1; i < SourcePoses.Num(); ++i)
	{
		BlendPosePerBone<ETransformBlendMode::Accumulate>(PerBoneIndices, BlendSampleDataCache[BlendSampleDataCacheIndices[i]], OutPose, SourcePoses[i]);
	}

	// Ensure that all of the resulting rotations are normalized
	OutPose.NormalizeRotations();

	if (SourceCurves.Num() > 0)
	{
		TArray<float>& SourceWeights = PerBoneScratch::SourceWeights;
		SourceWeights.SetNumUninitialized(BlendSampleDataCacheIndices.Num(), EAllowShrinking::No);
		for (int32 CacheIndex = 0; CacheIndex < BlendSampleDataCacheIndices.Num(); ++CacheIndex)
		{
			SourceWeights[CacheIndex] = BlendSampleDataCache[BlendSampleDataCacheIndices[CacheIndex]].TotalWeight;
		}

		BlendCurves(SourceCurves, SourceWeights, OutCurve);		
	}	

	if (SourceAttributes.Num() > 0)
	{
		UE::Anim::Attributes::BlendAttributesPerBone(SourceAttributes, PerBoneIndices, BlendSampleDataCache, BlendSampleDataCacheIndices, OutAttributes);
	}
}


void FAnimationRuntime::BlendPosesTogetherPerBoneInMeshSpace(
	const TArrayView<FCompactPose>           SourcePoses,
	const TArrayView<const FBlendedCurve>    SourceCurves,
	const UBlendSpace*                       BlendSpace,
	const TArrayView<const FBlendSampleData> BlendSampleDataCache,
	/*out*/ FCompactPose&                    ResultPose,
	/*out*/ FBlendedCurve&                   ResultCurve)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData = { ResultPose, ResultCurve, TempAttributes };

	BlendPosesTogetherPerBoneInMeshSpace(
		SourcePoses, SourceCurves, {}, BlendSpace, BlendSampleDataCache, AnimationPoseData);
}

void FAnimationRuntime::BlendPosesTogetherPerBoneInMeshSpace(
	TArrayView<FCompactPose>                             SourcePoses, 
	TArrayView<const FBlendedCurve>                      SourceCurves, 
	TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes, 
	const UBlendSpace*                                   BlendSpace, 
	TArrayView<const FBlendSampleData>                   BlendSampleDataCache, 
	/*out*/ FAnimationPoseData&                          OutAnimationPoseData)
{
	FCompactPose& OutPose = OutAnimationPoseData.GetPose();

	// Convert the sources poses into mesh space, and then once they have gone through
	// BlendPosesTogetherPerBone, convert back to local space

	// Convert SourcePoses.Rotation to be mesh space
	for (FCompactPose& Pose : SourcePoses)
	{
		for (const FCompactPoseBoneIndex BoneIndex : Pose.ForEachBoneIndex())
		{
			const FCompactPoseBoneIndex ParentIndex = Pose.GetParentBoneIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				FQuat NewRotation = Pose[ParentIndex].GetRotation() * Pose[BoneIndex].GetRotation();
				NewRotation.Normalize();
				Pose[BoneIndex].SetRotation(NewRotation);
			}
		}
	}

	// now we have mesh space rotation, call BlendPosesTogetherPerBone
	BlendPosesTogetherPerBone(SourcePoses, SourceCurves, SourceAttributes, BlendSpace, BlendSampleDataCache, OutAnimationPoseData);

	// Now OutPose has the output with mesh space rotation. Convert back to local space, start from back
	for (const FCompactPoseBoneIndex BoneIndex : OutPose.ForEachBoneIndexReverse())
	{
		const FCompactPoseBoneIndex ParentIndex = OutPose.GetParentBoneIndex(BoneIndex);
		if (ParentIndex != INDEX_NONE)
		{
			const FQuat LocalBlendQuat = OutPose[ParentIndex].GetRotation().Inverse() * OutPose[BoneIndex].GetRotation();
			OutPose[BoneIndex].SetRotation(LocalBlendQuat);
			OutPose[BoneIndex].NormalizeRotation();
		}
	}
}

void FAnimationRuntime::LerpPoses(FCompactPose& PoseA, const FCompactPose& PoseB, FBlendedCurve& CurveA, const FBlendedCurve& CurveB, float Alpha)
{
	// If pose A is full weight, we're set.
	if (FAnimWeight::IsRelevant(Alpha))
	{
		// Make sure poses are compatible with each other.
		check(&PoseA.GetBoneContainer() == &PoseB.GetBoneContainer());

		// If pose 2 is full weight, just copy, no need to blend.
		if (FAnimWeight::IsFullWeight(Alpha))
		{
			PoseA.CopyBonesFrom(PoseB);
			CurveA.CopyFrom(CurveB);
		}
		else
		{
			const ScalarRegister VWeightOfPose1(1.f - Alpha);
			const ScalarRegister VWeightOfPose2(Alpha);
			for (FCompactPoseBoneIndex BoneIndex : PoseA.ForEachBoneIndex())
			{
				FTransform& InOutBoneTransform1 = PoseA[BoneIndex];
				InOutBoneTransform1 *= VWeightOfPose1;

				const FTransform& BoneTransform2 = PoseB[BoneIndex];
				InOutBoneTransform1.AccumulateWithShortestRotation(BoneTransform2, VWeightOfPose2);

				InOutBoneTransform1.NormalizeRotation();
			}

			CurveA.LerpTo(CurveB, Alpha);
		}
	}
}

void FAnimationRuntime::LerpPosesPerBone(FCompactPose& PoseA, const FCompactPose& PoseB, FBlendedCurve& CurveA, const FBlendedCurve& CurveB, float Alpha, const TArray<float>& PerBoneWeights)
{
	// If pose A is full weight, we're set.
	if (FAnimWeight::IsRelevant(Alpha))
	{
		// Make sure poses are compatible with each other.
		check(&PoseA.GetBoneContainer() == &PoseB.GetBoneContainer());

		for (FCompactPoseBoneIndex BoneIndex : PoseA.ForEachBoneIndex())
		{
			const float BoneAlpha = Alpha * PerBoneWeights[BoneIndex.GetInt()];
			if (FAnimWeight::IsRelevant(BoneAlpha))
			{
				const ScalarRegister VWeightOfPose1(1.f - BoneAlpha);
				const ScalarRegister VWeightOfPose2(BoneAlpha);

				FTransform& InOutBoneTransform1 = PoseA[BoneIndex];
				InOutBoneTransform1 *= VWeightOfPose1;

				const FTransform& BoneTransform2 = PoseB[BoneIndex];
				InOutBoneTransform1.AccumulateWithShortestRotation(BoneTransform2, VWeightOfPose2);

				InOutBoneTransform1.NormalizeRotation();
			}
		}

		// @note : This isn't perfect as curve can link to joint, and it would be the best to use that information
		// but that is very expensive option as we have to have another indirect look up table to search. 
		// For now, replacing with combine (non-zero will be overridden)
		// in the future, we might want to do this outside if we want per bone blend to apply curve also UE-39182
		CurveA.Combine(CurveB);
	}
}

void FAnimationRuntime::LerpPosesWithBoneIndexList(FCompactPose& PoseA, const FCompactPose& PoseB, FBlendedCurve& CurveA, const FBlendedCurve& CurveB, float Alpha, const TArray<FCompactPoseBoneIndex>& BoneIndices)
{
	// If pose A is full weight, we're set.
	if (FAnimWeight::IsRelevant(Alpha))
	{
		// Make sure poses are compatible with each other.
		check(&PoseA.GetBoneContainer() == &PoseB.GetBoneContainer());

		// If pose 2 is full weight, just copy, no need to blend.
		if (FAnimWeight::IsFullWeight(Alpha))
		{
			for (int32 Index = 0; Index < BoneIndices.Num(); Index++)
			{
				const FCompactPoseBoneIndex& BoneIndex = BoneIndices[Index];
				PoseA[BoneIndex] = PoseB[BoneIndex];
			}

			CurveA.CopyFrom(CurveB);
		}
		else
		{
			const ScalarRegister VWeightOfPose1(1.f - Alpha);
			const ScalarRegister VWeightOfPose2(Alpha);

			for (int32 Index = 0; Index < BoneIndices.Num(); Index++)
			{
				const FCompactPoseBoneIndex& BoneIndex = BoneIndices[Index];

				FTransform& InOutBoneTransform1 = PoseA[BoneIndex];
				InOutBoneTransform1 *= VWeightOfPose1;

				const FTransform& BoneTransform2 = PoseB[BoneIndex];
				InOutBoneTransform1.AccumulateWithShortestRotation(BoneTransform2, VWeightOfPose2);
				InOutBoneTransform1.NormalizeRotation();
			}

			// @note : This isn't perfect as curve can link to joint, and it would be the best to use that information
			// but that is very expensive option as we have to have another indirect look up table to search. 
			// For now, replacing with combine (non-zero will be overridden)
			// in the future, we might want to do this outside if we want per bone blend to apply curve also UE-39182
			CurveA.Combine(CurveB);
		}
	}
}

void FAnimationRuntime::LerpBoneTransforms(TArray<FTransform>& A, const TArray<FTransform>& B, float Alpha, const TArray<FBoneIndexType>& RequiredBonesArray)
{
	if (FAnimWeight::IsFullWeight(Alpha))
	{
		A = B;
		return;
	}

	if (!FAnimWeight::IsRelevant(Alpha))
	{
		return;
	}

#if INTEL_ISPC
	if (bAnim_LerpBoneTransforms_ISPC_Enabled)
	{
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				A.GetData(),
				A.Num() * sizeof(FTransform));
			ispc::LerpBoneTransforms(
				reinterpret_cast<ispc::FTransform*>(A.GetData()),
				reinterpret_cast<const ispc::FTransform*>(B.GetData()),
				Alpha,
				RequiredBonesArray.GetData(),
				RequiredBonesArray.Num());
		};
	}
	else
#endif
	{
		FTransform* ATransformData = A.GetData();
		const FTransform* BTransformData = B.GetData();
		const ScalarRegister VAlpha(Alpha);
		const ScalarRegister VOneMinusAlpha(1.f - Alpha);

		for (int32 Index=0; Index<RequiredBonesArray.Num(); Index++)
		{
			const int32& BoneIndex = RequiredBonesArray[Index];
			FTransform* TA = ATransformData + BoneIndex;
			const FTransform* TB = BTransformData + BoneIndex;

			*TA *= VOneMinusAlpha;
			TA->AccumulateWithShortestRotation(*TB, VAlpha);
			TA->NormalizeRotation();
		}
	}
}

void FAnimationRuntime::BlendTransformsByWeight(FTransform& OutTransform, const TArray<FTransform>& Transforms, const TArray<float>& Weights)
{
	int32 NumBlends = Transforms.Num();
	check(Transforms.Num() == Weights.Num());

	if (NumBlends == 0)
	{
		OutTransform = FTransform::Identity;
	}
	else if (NumBlends == 1)
	{
		OutTransform = Transforms[0];
	}
	else
	{
		// @todo : change this to be veoctorized or move to Ftransform
		FVector		OutTranslation = Transforms[0].GetTranslation() * Weights[0];
		FQuat		OutRotation = Transforms[0].GetRotation() * Weights[0];
		FVector		OutScale = Transforms[0].GetScale3D() * Weights[0];

		// otherwise we just purely blend by number, and then later we normalize
		for (int32 Index = 1; Index < NumBlends; ++Index)
		{
			// Simple linear interpolation for translation and scale.
			OutTranslation = FMath::Lerp(OutTranslation, Transforms[Index].GetTranslation(), Weights[Index]);
			OutScale = FMath::Lerp(OutScale, Transforms[Index].GetScale3D(), Weights[Index]);
			OutRotation = FQuat::FastLerp(OutRotation, Transforms[Index].GetRotation(), Weights[Index]);
		}

		OutRotation.Normalize();
		OutTransform = FTransform(OutRotation, OutTranslation, OutScale);
	}
}

void FAnimationRuntime::CombineWithAdditiveAnimations(int32 NumAdditivePoses, const FTransformArrayA2** SourceAdditivePoses, const float* SourceAdditiveWeights, const FBoneContainer& RequiredBones, /*inout*/ FTransformArrayA2& Atoms)
{
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	for (int32 PoseIndex = 0; PoseIndex < NumAdditivePoses; ++PoseIndex)
	{
		const ScalarRegister VBlendWeight(SourceAdditiveWeights[PoseIndex]);
		const FTransformArrayA2& SourceAtoms = *SourceAdditivePoses[PoseIndex];

		for (int32 j = 0; j < RequiredBoneIndices.Num(); ++j)
		{
			const int32 BoneIndex = RequiredBoneIndices[j];
			FTransform SourceAtom = SourceAtoms[BoneIndex];
			FTransform::BlendFromIdentityAndAccumulate(Atoms[BoneIndex], SourceAtom, VBlendWeight);
		}
	}
}

void FAnimationRuntime::ConvertTransformToAdditive(FTransform& TargetTransform, const FTransform& BaseTransform)
{
	TargetTransform.SetRotation(TargetTransform.GetRotation() * BaseTransform.GetRotation().Inverse());
	TargetTransform.SetTranslation(TargetTransform.GetTranslation() - BaseTransform.GetTranslation());
	// additive scale considers how much it grow or lower
	// in order to support blending between different additive scale, we save [(target scale)/(source scale) - 1.f], and this can blend with 
	// other delta scale value
	// when we apply to the another scale, we apply scale * (1 + [additive scale])
	TargetTransform.SetScale3D(TargetTransform.GetScale3D() * BaseTransform.GetSafeScaleReciprocal(BaseTransform.GetScale3D()) - 1.f);
	TargetTransform.NormalizeRotation();
}

void FAnimationRuntime::ConvertPoseToAdditive(FCompactPose& TargetPose, const FCompactPose& BasePose)
{
#if INTEL_ISPC
	if (bAnim_ConvertPoseToAdditive_ISPC_Enabled)
	{
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				TargetPose.GetMutableBones().GetData(),
				TargetPose.GetNumBones() * sizeof(FTransform));
			ispc::ConvertPoseToAdditive(
				reinterpret_cast<ispc::FTransform*>(TargetPose.GetMutableBones().GetData()),
				reinterpret_cast<const ispc::FTransform*>(BasePose.GetBones().GetData()),
				BasePose.GetNumBones());
		};
	}
	else
#endif
	{
		for (FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
		{
			FTransform& TargetTransform = TargetPose[BoneIndex];
			const FTransform& BaseTransform = BasePose[BoneIndex];

			ConvertTransformToAdditive(TargetTransform, BaseTransform);
		}
	}
}

void FAnimationRuntime::ConvertPoseToMeshRotation(FCompactPose& LocalPose)
{
	SCOPE_CYCLE_COUNTER(STAT_ConvertPoseToMeshRot);

#if INTEL_ISPC
	if (bAnim_ConvertPoseToMeshRotation_ISPC_Enabled)
	{
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				LocalPose.GetMutableBones().GetData(),
				LocalPose.GetNumBones() * sizeof(FTransform));
			ispc::ConvertPoseToMeshRotation(
				reinterpret_cast<ispc::FTransform*>(LocalPose.GetMutableBones().GetData()),
				reinterpret_cast<const int32*>(LocalPose.GetBoneContainer().GetCompactPoseParentBoneArray().GetData()),
				LocalPose.GetNumBones());
		};
	}
	else
#endif
	{
		for (FCompactPoseBoneIndex BoneIndex(1); BoneIndex < LocalPose.GetNumBones(); ++BoneIndex)
		{
			const FCompactPoseBoneIndex ParentIndex = LocalPose.GetParentBoneIndex(BoneIndex);
	
			const FQuat MeshSpaceRotation = LocalPose[ParentIndex].GetRotation() * LocalPose[BoneIndex].GetRotation();
			LocalPose[BoneIndex].SetRotation(MeshSpaceRotation);
		}
	}
}

void FAnimationRuntime::ConvertMeshRotationPoseToLocalSpace(FCompactPose& Pose)
{
	SCOPE_CYCLE_COUNTER(STAT_ConvertMeshRotPoseToLocalSpace);

#if INTEL_ISPC
	if (bAnim_ConvertMeshRotationPoseToLocalSpace_ISPC_Enabled)
	{
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				Pose.GetMutableBones().GetData(),
				Pose.GetNumBones() * sizeof(FTransform));
			ispc::ConvertMeshRotationPoseToLocalSpace(
				reinterpret_cast<ispc::FTransform*>(Pose.GetMutableBones().GetData()),
				reinterpret_cast<const int32*>(Pose.GetBoneContainer().GetCompactPoseParentBoneArray().GetData()),
				Pose.GetNumBones());
		};
	}
	else
#endif
	{
		for (FCompactPoseBoneIndex BoneIndex(Pose.GetNumBones() - 1); BoneIndex > 0; --BoneIndex)
		{
			const FCompactPoseBoneIndex ParentIndex = Pose.GetParentBoneIndex(BoneIndex);

			FQuat LocalSpaceRotation = Pose[ParentIndex].GetRotation().Inverse() * Pose[BoneIndex].GetRotation();
			Pose[BoneIndex].SetRotation(LocalSpaceRotation);
		}
	}
}

void FAnimationRuntime::AccumulateAdditivePose(FCompactPose& BasePose, const FCompactPose& AdditivePose, FBlendedCurve& BaseCurve, const FBlendedCurve& AdditiveCurve, float Weight, enum EAdditiveAnimationType AdditiveType)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData BaseAnimationPoseData = { BasePose, BaseCurve, TempAttributes };
	const FAnimationPoseData AdditiveAnimationPoseData(const_cast<FCompactPose&>(AdditivePose), const_cast<FBlendedCurve&>(AdditiveCurve), TempAttributes);

	AccumulateAdditivePose(BaseAnimationPoseData, AdditiveAnimationPoseData, Weight, AdditiveType);
}


void FAnimationRuntime::AccumulateLocalSpaceAdditivePose(FCompactPose& BasePose, const FCompactPose& AdditivePose, FBlendedCurve& BaseCurve, const FBlendedCurve& AdditiveCurve, float Weight)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData BaseAnimationPoseData = { BasePose, BaseCurve, TempAttributes };
	const FAnimationPoseData AdditiveAnimationPoseData(const_cast<FCompactPose&>(AdditivePose), const_cast<FBlendedCurve&>(AdditiveCurve), TempAttributes);

	AccumulateAdditivePose(BaseAnimationPoseData, AdditiveAnimationPoseData, Weight, EAdditiveAnimationType::AAT_LocalSpaceBase);
}

void FAnimationRuntime::AccumulateLocalSpaceAdditivePose(FAnimationPoseData& BaseAnimationPoseData, const FAnimationPoseData& AdditiveAnimationPoseData, float Weight)
{
	AccumulateAdditivePose(BaseAnimationPoseData, AdditiveAnimationPoseData, Weight, EAdditiveAnimationType::AAT_LocalSpaceBase);
}

void FAnimationRuntime::AccumulateMeshSpaceRotationAdditiveToLocalPose(FCompactPose& BasePose, const FCompactPose& MeshSpaceRotationAdditive, FBlendedCurve& BaseCurve, const FBlendedCurve& AdditiveCurve, float Weight)
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData BaseAnimationPoseData = { BasePose, BaseCurve, TempAttributes };
	const FAnimationPoseData AdditiveAnimationPoseData(const_cast<FCompactPose&>(MeshSpaceRotationAdditive), const_cast<FBlendedCurve&>(AdditiveCurve), TempAttributes);

	AccumulateAdditivePose(BaseAnimationPoseData, AdditiveAnimationPoseData, Weight, EAdditiveAnimationType::AAT_RotationOffsetMeshSpace);
}

void FAnimationRuntime::AccumulateMeshSpaceRotationAdditiveToLocalPose(FAnimationPoseData& BaseAnimationPoseData, const FAnimationPoseData& MeshSpaceRotationAdditiveAnimationPoseData, float Weight)
{
	AccumulateAdditivePose(BaseAnimationPoseData, MeshSpaceRotationAdditiveAnimationPoseData, Weight, EAdditiveAnimationType::AAT_RotationOffsetMeshSpace);
}

void FAnimationRuntime::AccumulateRootSpaceRotationAdditiveToLocalPose(FAnimationPoseData& BaseAnimationPoseData, const FAnimationPoseData& RootSpaceRotationAdditiveAnimationPoseData, float Weight)
{
	AccumulateRootSpaceAdditivePoseInternal(BaseAnimationPoseData.GetPose(), RootSpaceRotationAdditiveAnimationPoseData.GetPose(), Weight);

	// if curve exists, accumulate with the weight, 
	BaseAnimationPoseData.GetCurve().Accumulate(RootSpaceRotationAdditiveAnimationPoseData.GetCurve(), Weight);

	UE::Anim::Attributes::AccumulateAttributes(RootSpaceRotationAdditiveAnimationPoseData.GetAttributes(), BaseAnimationPoseData.GetAttributes(), Weight, EAdditiveAnimationType::AAT_RotationOffsetMeshSpace);

	// normalize
	BaseAnimationPoseData.GetPose().NormalizeRotations();
}

void FAnimationRuntime::AccumulateAdditivePose(FAnimationPoseData& BaseAnimationPoseData, const FAnimationPoseData& AdditiveAnimationPoseData, float Weight, enum EAdditiveAnimationType AdditiveType)
{
	if (AdditiveType == AAT_RotationOffsetMeshSpace)
	{
		AccumulateMeshSpaceRotationAdditiveToLocalPoseInternal(BaseAnimationPoseData.GetPose(), AdditiveAnimationPoseData.GetPose(), Weight);
	}
	else
	{
		AccumulateLocalSpaceAdditivePoseInternal(BaseAnimationPoseData.GetPose(), AdditiveAnimationPoseData.GetPose(), Weight);
	}

	// if curve exists, accumulate with the weight, 
	BaseAnimationPoseData.GetCurve().Accumulate(AdditiveAnimationPoseData.GetCurve(), Weight);

	UE::Anim::Attributes::AccumulateAttributes(AdditiveAnimationPoseData.GetAttributes(), BaseAnimationPoseData.GetAttributes(), Weight, AdditiveType);
	
	// normalize
	BaseAnimationPoseData.GetPose().NormalizeRotations();
}

void FAnimationRuntime::AccumulateLocalSpaceAdditivePoseInternal(FCompactPose& BasePose, const FCompactPose& AdditivePose, float Weight)
{
	if (FAnimWeight::IsRelevant(Weight))
	{
		const ScalarRegister VBlendWeight(Weight);
		if (FAnimWeight::IsFullWeight(Weight))
		{
#if INTEL_ISPC
			if (bAnim_AccumulateLocalSpaceAdditivePose_ISPC_Enabled)
			{
				UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
				{
					AutoRTFM::RecordOpenWrite(
						BasePose.GetMutableBones().GetData(),
						BasePose.GetNumBones() * sizeof(FTransform));
					ispc::AccumulateWithAdditiveScale(
						reinterpret_cast<ispc::FTransform*>(BasePose.GetMutableBones().GetData()),
						reinterpret_cast<const ispc::FTransform*>(AdditivePose.GetBones().GetData()),
						Weight,
						BasePose.GetNumBones());
				};
			}
			else
#endif
			{
				// fast path, no need to weight additive.
				for (FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
				{
					BasePose[BoneIndex].AccumulateWithAdditiveScale(AdditivePose[BoneIndex], VBlendWeight);
				}
			}
		}
		else
		{
			// Slower path w/ weighting
			for (FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
			{
				// copy additive, because BlendFromIdentityAndAccumulate modifies it.
				FTransform Additive = AdditivePose[BoneIndex];
				FTransform::BlendFromIdentityAndAccumulate(BasePose[BoneIndex], Additive, VBlendWeight);
			}
		}
	}
}

void FAnimationRuntime::AccumulateMeshSpaceRotationAdditiveToLocalPoseInternal(FCompactPose& BasePose, const FCompactPose& MeshSpaceRotationAdditive, float Weight)
{
	SCOPE_CYCLE_COUNTER(STAT_AccumulateMeshSpaceRotAdditiveToLocalPose);

	if (FAnimWeight::IsRelevant(Weight))
	{
		// Convert base pose from local space to mesh space rotation.
		FAnimationRuntime::ConvertPoseToMeshRotation(BasePose);

		// Add MeshSpaceRotAdditive to it
		FAnimationRuntime::AccumulateLocalSpaceAdditivePoseInternal(BasePose, MeshSpaceRotationAdditive, Weight);

		// Convert back to local space
		FAnimationRuntime::ConvertMeshRotationPoseToLocalSpace(BasePose);
	}
}

ENGINE_API void FAnimationRuntime::AccumulateRootSpaceAdditivePoseInternal(FCompactPose& BasePose, const FCompactPose& RootSpaceAdditivePose, float Weight)
{
	if (FAnimWeight::IsRelevant(Weight))
	{		
		// Cache off the Root Transform and Zero it Out
		FCompactPoseBoneIndex BoneIndex(0);
		FTransform RootTransform = BasePose[BoneIndex];
		BasePose[BoneIndex] = FTransform::Identity;

		// Convert base pose from local space to mesh space rotation.
		FAnimationRuntime::ConvertPoseToMeshRotation(BasePose);

		// Add MeshSpaceRotAdditive to it
		FAnimationRuntime::AccumulateLocalSpaceAdditivePoseInternal(BasePose, RootSpaceAdditivePose, Weight);

		// Convert back to local space
		FAnimationRuntime::ConvertMeshRotationPoseToLocalSpace(BasePose);

		// Restore the original Root Transform
		BasePose[BoneIndex] = RootTransform;
	}
}

void FAnimationRuntime::MirrorCurves(FBlendedCurve& Curves, const UMirrorDataTable& MirrorDataTable)
{
	for(const TPair<FName, FName>& MirrorPair : MirrorDataTable.CurveToMirrorCurveMap)
	{
		Curves.Mirror(MirrorPair.Key, MirrorPair.Value);
	}
}

FVector FAnimationRuntime::MirrorVector(const FVector& V, EAxis::Type MirrorAxis)
{
	FVector MirrorV(V);

	switch (MirrorAxis)
	{
	case EAxis::X:
		MirrorV.X = -MirrorV.X;
		break;
	case EAxis::Y:
		MirrorV.Y = -MirrorV.Y;
		break;
	case EAxis::Z:
		MirrorV.Z = -MirrorV.Z;
		break;
	}

	return MirrorV;
}

FQuat FAnimationRuntime::MirrorQuat(const FQuat& Q, EAxis::Type MirrorAxis)
{
	FQuat MirrorQ(Q);

	// Given an axis V and an angle A, the corresponding unmirrored quaternion Q = { Q.XYZ, Q.W } is:
	//
	//		Q = { V * sin(A/2), cos(A/2) }
	//
	//  mirror both the axis of rotation and the angle of rotation around that axis.
	// Therefore, the mirrored quaternion Q' for the axis V and angle A is:
	//
	//		Q' = { MirrorVector(V) * sin(-A/2), cos(-A/2) }
	//		Q' = { -MirrorVector(V) * sin(A/2), cos(A/2) }
	//		Q' = { -MirrorVector(V * sin(A/2)), cos(A/2) }
	//		Q' = { -MirrorVector(Q.XYZ), Q.W }
	//
	switch (MirrorAxis)
	{
	case EAxis::X:
		MirrorQ.Y = -MirrorQ.Y;
		MirrorQ.Z = -MirrorQ.Z;
		break;
	case EAxis::Y:
		MirrorQ.X = -MirrorQ.X;
		MirrorQ.Z = -MirrorQ.Z;
		break;
	case EAxis::Z:
		MirrorQ.X = -MirrorQ.X;
		MirrorQ.Y = -MirrorQ.Y;
		break;
	}

	return MirrorQ;
}


void FAnimationRuntime::MirrorPose(FCompactPose& Pose, EAxis::Type MirrorAxis, const TArray<FCompactPoseBoneIndex>& CompactPoseMirrorBones, const TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex>& ComponentSpaceRefRotations)
{
	const FBoneContainer& BoneContainer = Pose.GetBoneContainer();
	if (MirrorAxis == EAxis::None)
	{
		return;
	}

	// Mirroring is authored in object space and as such we must transform the local space transforms in object space in order
	// to apply the object space mirroring axis. To facilitate this, we use object space transforms for the bind pose which can be cached.
	// We ignore the translation/scale part of the bind pose as they don't impact mirroring.
	// 
	// Rotations, translations, and scales are all treated differently:
	//    Rotation:
	//        We transform the local space rotation into object space
	//        We mirror the rotation axis
	//        We apply a correction: if the source and target bones are different, we must account for the mirrored delta between the two
	//        We transform the result back into local space
	//    Translation:
	//        We rotate the local space translation into object space
	//        We mirror the result
	//        We then rotate it back into local space
	//    Scale:
	//        Mirroring does not modify scale
	// 
	// This sadly doesn't quite work for additive poses because in order to transform it into the bind pose reference frame,
	// we need the base pose it is applied on. Worse still, the base pose might not be static, it could be a time scaled sequence.

	auto MirrorTransform = [&ComponentSpaceRefRotations, MirrorAxis](const FTransform& SourceTransform, const FCompactPoseBoneIndex& SourceParentIndex, const FCompactPoseBoneIndex& SourceBoneIndex, const FCompactPoseBoneIndex& TargetParentIndex, const FCompactPoseBoneIndex& TargetBoneIndex) -> FTransform {

		const FQuat TargetParentRefRotation = ComponentSpaceRefRotations[TargetParentIndex];
		const FQuat TargetBoneRefRotation = ComponentSpaceRefRotations[TargetBoneIndex];
		const FQuat SourceParentRefRotation = ComponentSpaceRefRotations[SourceParentIndex];
		const FQuat SourceBoneRefRotation = ComponentSpaceRefRotations[SourceBoneIndex];

		// Mirror the translation component:  Rotate the translation into the space of the mirror plane,  mirror across the mirror plane, and rotate into the space of its new parent

		FVector T = SourceTransform.GetTranslation();
		T = SourceParentRefRotation.RotateVector(T);
		T = MirrorVector(T, MirrorAxis);
		T = TargetParentRefRotation.UnrotateVector(T);

		// Mirror the rotation component:- Rotate into the space of the mirror plane, mirror across the plane, apply corrective rotation to align result with target space's rest orientation, 
		// then rotate into the space of its new parent

		FQuat Q = SourceTransform.GetRotation();
		Q = SourceParentRefRotation * Q;
		Q = MirrorQuat(Q, MirrorAxis);
		Q *= MirrorQuat(SourceBoneRefRotation, MirrorAxis).Inverse() * TargetBoneRefRotation;
		Q = TargetParentRefRotation.Inverse() * Q;

		FVector S = SourceTransform.GetScale3D();

		return FTransform(Q, T, S);
	};

	// Mirror the root bone
	{
		const FCompactPoseBoneIndex RootBoneIndex(0);
		const FCompactPoseBoneIndex MirrorRootBoneIndex = CompactPoseMirrorBones[RootBoneIndex.GetInt()];
		if (MirrorRootBoneIndex.IsValid())
		{
			const FQuat RootBoneRefRotation = ComponentSpaceRefRotations[RootBoneIndex];

			FVector T = Pose[RootBoneIndex].GetTranslation();
			T = MirrorVector(T, MirrorAxis);

			FQuat Q = Pose[RootBoneIndex].GetRotation();
			Q = MirrorQuat(Q, MirrorAxis);
			Q *= MirrorQuat(RootBoneRefRotation, MirrorAxis).Inverse() * RootBoneRefRotation;

			FVector S = Pose[RootBoneIndex].GetScale3D();

			Pose[RootBoneIndex] = FTransform(Q, T, S);
		}
	}

	const int32 NumBones = BoneContainer.GetCompactPoseNumBones();

	// Mirror the non-root bones
	for (FCompactPoseBoneIndex TargetBoneIndex(1); TargetBoneIndex < NumBones; ++TargetBoneIndex)
	{
		const FCompactPoseBoneIndex SourceBoneIndex = CompactPoseMirrorBones[TargetBoneIndex.GetInt()];
		if (SourceBoneIndex == TargetBoneIndex)
		{
			const FCompactPoseBoneIndex TargetParentIndex = BoneContainer.GetParentBoneIndex(TargetBoneIndex);
			Pose[TargetBoneIndex] = MirrorTransform(Pose[TargetBoneIndex], TargetParentIndex, TargetBoneIndex, TargetParentIndex, TargetBoneIndex);
		}
		else if (SourceBoneIndex > TargetBoneIndex)
		{
			const FCompactPoseBoneIndex TargetParentIndex = BoneContainer.GetParentBoneIndex(TargetBoneIndex);
			const FCompactPoseBoneIndex SourceParentIndex = BoneContainer.GetParentBoneIndex(SourceBoneIndex);
			const FTransform NewTargetBoneTransform = MirrorTransform(Pose[SourceBoneIndex], SourceParentIndex, SourceBoneIndex, TargetParentIndex, TargetBoneIndex);
			const FTransform NewSourceBoneTransform = MirrorTransform(Pose[TargetBoneIndex], TargetParentIndex, TargetBoneIndex, SourceParentIndex, SourceBoneIndex);
			Pose[TargetBoneIndex] = NewTargetBoneTransform;
			Pose[SourceBoneIndex] = NewSourceBoneTransform;
		}
	}
}

void FAnimationRuntime::MirrorPose(FCompactPose& Pose, const UMirrorDataTable& MirrorDataTable)
{
	const FBoneContainer& BoneContainer = Pose.GetBoneContainer();
	USkeleton* Skeleton = BoneContainer.GetSkeletonAsset();
	if (Skeleton)
	{
		TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex> MirrorBoneIndexes;
		MirrorDataTable.FillMirrorBoneIndexes(Skeleton, MirrorBoneIndexes);

		// Compact pose format of Mirror Bone Map
		TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex> CompactPoseMirrorBones;
		MirrorDataTable.FillCompactPoseMirrorBones(BoneContainer, MirrorBoneIndexes, CompactPoseMirrorBones);

		const int32 NumBones = BoneContainer.GetCompactPoseNumBones();

		TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex> ComponentSpaceRefRotations;
		ComponentSpaceRefRotations.SetNumUninitialized(NumBones);
		ComponentSpaceRefRotations[FCompactPoseBoneIndex(0)] = BoneContainer.GetRefPoseTransform(FCompactPoseBoneIndex(0)).GetRotation();
		for (FCompactPoseBoneIndex BoneIndex(1); BoneIndex < NumBones; ++BoneIndex)
		{
			const FCompactPoseBoneIndex ParentBoneIndex = BoneContainer.GetParentBoneIndex(BoneIndex);
			ComponentSpaceRefRotations[BoneIndex] = ComponentSpaceRefRotations[ParentBoneIndex] * BoneContainer.GetRefPoseTransform(BoneIndex).GetRotation();
		}
		MirrorPose(Pose, MirrorDataTable.MirrorAxis, CompactPoseMirrorBones, ComponentSpaceRefRotations);
	}
}

/** 
 * return ETypeAdvanceAnim type
 */
ETypeAdvanceAnim FAnimationRuntime::AdvanceTime(const bool bAllowLooping, const float MoveDelta, float& InOutTime, const float EndTime)
{
	float NewTime = InOutTime + MoveDelta;

	if (NewTime < 0.f || NewTime > EndTime)
	{
		if (bAllowLooping)
		{
			if (EndTime != 0.f)
			{
				NewTime = FMath::Fmod(NewTime, EndTime);
				// Fmod doesn't give result that falls into (0, EndTime), but one that falls into (-EndTime, EndTime). Negative values need to be handled in custom way
				if (NewTime < 0.f)
				{
					NewTime += EndTime;
				}
			}
			else
			{
				// end time is 0.f
				NewTime = 0.f;
			}

			// it has been looped
			InOutTime = NewTime;
			return ETAA_Looped;
		}
		else 
		{
			// If not, snap time to end of sequence and stop playing.
			InOutTime = FMath::Clamp(NewTime, 0.f, EndTime);
			return ETAA_Finished;
		}
	}

	InOutTime = NewTime;
	return ETAA_Default;
}

/** 
 * Scale transforms by Weight.
 * Result is obviously NOT normalized.
 */
void FAnimationRuntime::ApplyWeightToTransform(const FBoneContainer& RequiredBones, /*inout*/ FTransformArrayA2& Atoms, float Weight)
{
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	ScalarRegister MultWeight(Weight);
	for (int32 j = 0; j < RequiredBoneIndices.Num(); ++j)
	{
		const int32 BoneIndex = RequiredBoneIndices[j];
		Atoms[BoneIndex] *= MultWeight;
	}			
}

/* from % from OutKeyIndex1, meaning (CurrentKeyIndex(double)-OutKeyIndex1)/(OutKeyIndex2-OutKeyIndex1) */
void FAnimationRuntime::GetKeyIndicesFromTime(int32& OutKeyIndex1, int32& OutKeyIndex2, float& OutAlpha, const double Time, const int32 NumKeys, const double SequenceLength, double FramesPerSecond)
{
	// Check for 1-frame, before-first-frame and after-last-frame cases.
	if (Time <= 0.0 || NumKeys == 1)
	{
		OutKeyIndex1 = 0;
		OutKeyIndex2 = 0;
		OutAlpha = 0.0f;
		return;
	}

	const int32 LastIndex = NumKeys - 1;
	if (Time >= SequenceLength)
	{
		OutKeyIndex1 = LastIndex;
		OutKeyIndex2 = 0;
		OutAlpha = 0.0f;
		return;
	}

	// Calulate the frames per second if we didn't provide any.
	if (FramesPerSecond <= 0.0)
	{
		const int32 NumFrames = NumKeys - 1;
		FramesPerSecond = NumFrames / SequenceLength;
	}

	const double KeyPos = Time * FramesPerSecond;

	// Find the integer part (ensuring within range) and that gives us the 'starting' key index.
	const int32 KeyIndex1 = FMath::Clamp<int32>( FMath::FloorToInt(KeyPos), 0, NumKeys - 1 );  // @todo should be changed to FMath::TruncToInt

	// The alpha (fractional part) is then just the remainder.
	const double Alpha = KeyPos - (double)KeyIndex1;

	int32 KeyIndex2 = KeyIndex1 + 1;

	// If we have gone over the end, do different things in case of looping
	if (KeyIndex2 == NumKeys)
	{
		KeyIndex2 = KeyIndex1;
	}

	OutKeyIndex1 = KeyIndex1;
	OutKeyIndex2 = KeyIndex2;
	OutAlpha = (float)Alpha;
}

void FAnimationRuntime::GetKeyIndicesFromTime(int32& OutKeyIndex1, int32& OutKeyIndex2, float& OutAlpha, const double Time, const FFrameRate& FrameRate, const int32 NumberOfKeys)
{
	// Check for 1-frame, before-first-frame and after-last-frame cases.
	if (Time <= 0.0 || NumberOfKeys == 1)
	{
		OutKeyIndex1 = 0;
		OutKeyIndex2 = 0;
		OutAlpha = 0.0f;
		return;
	}

	const FFrameTime FrameTime = FrameRate.AsFrameTime(Time);
	const FFrameTime LastFrameTimeIndex = FFrameTime(NumberOfKeys - 1);
	if (FrameTime >= LastFrameTimeIndex)
	{
		OutKeyIndex1 = LastFrameTimeIndex.FrameNumber.Value;
		OutKeyIndex2 = 0;
		OutAlpha = 0.0f;
		return;
	}

	// Find the integer part (ensuring within range) and that gives us the 'starting' key index.
	const int32 KeyIndex1 = FMath::Clamp<int32>(FrameTime.GetFrame().Value, 0, NumberOfKeys - 1); 

	// The alpha (fractional part) is then just the remainder.
	const float Alpha = FrameTime.GetSubFrame();

	int32 KeyIndex2 = KeyIndex1 + 1;

	// If we have gone over the end, do different things in case of looping
	if (KeyIndex2 == NumberOfKeys)
	{
		KeyIndex2 = KeyIndex1;
	}

	OutKeyIndex1 = KeyIndex1;
	OutKeyIndex2 = KeyIndex2;
	OutAlpha = Alpha;
}

FTransform FAnimationRuntime::GetComponentSpaceRefPose(const FCompactPoseBoneIndex& CompactPoseBoneIndex, const FBoneContainer& BoneContainer)
{
	FCompactPoseBoneIndex CurrentIndex = CompactPoseBoneIndex;
	FTransform CSTransform = FTransform::Identity;
	while (CurrentIndex.GetInt() != INDEX_NONE)
	{
		CSTransform *= BoneContainer.GetRefPoseTransform(CurrentIndex);
		CurrentIndex = BoneContainer.GetParentBoneIndex(CurrentIndex);
	}
	
	return CSTransform;
}

void FAnimationRuntime::FillWithRefPose(TArray<FTransform>& OutAtoms, const FBoneContainer& RequiredBones)
{
	// Copy Target Asset's ref pose.
	OutAtoms = RequiredBones.GetRefPoseArray();

	// If retargeting is disabled, copy ref pose from Skeleton, rather than mesh.
	// this is only used in editor and for debugging.
	if( RequiredBones.GetDisableRetargeting() )
	{
		checkSlow( RequiredBones.IsValid() );
		// Only do this if we have a mesh. otherwise we're not retargeting animations.
		if( RequiredBones.GetSkeletalMeshAsset() )
		{
			TArray<FBoneIndexType> const& RequireBonesIndexArray = RequiredBones.GetBoneIndicesArray();
			TArray<FTransform> const& SkeletonRefPose = RequiredBones.GetSkeletonAsset()->GetRefLocalPoses();

			for (int32 ArrayIndex = 0; ArrayIndex<RequireBonesIndexArray.Num(); ArrayIndex++)
			{
				int32 const PoseBoneIndex = RequireBonesIndexArray[ArrayIndex];
				FSkeletonPoseBoneIndex const SkeletonBoneIndex = RequiredBones.GetSkeletonPoseIndexFromMeshPoseIndex(FMeshPoseBoneIndex(PoseBoneIndex));

				// Pose bone index should always exist in Skeleton
				checkSlow(SkeletonBoneIndex.IsValid());
				OutAtoms[PoseBoneIndex] = SkeletonRefPose[SkeletonBoneIndex.GetInt()];
			}
		}
	}
}

void FAnimationRuntime::ConvertPoseToMeshSpace(const TArray<FTransform>& LocalTransforms, TArray<FTransform>& MeshSpaceTransforms, const FBoneContainer& RequiredBones)
{
	const int32 NumBones = RequiredBones.GetNumBones();

	// right now all this does is to convert to SpaceBases
	check( NumBones == LocalTransforms.Num() );
	check( NumBones == MeshSpaceTransforms.Num() );

	const FTransform* LocalTransformsData = LocalTransforms.GetData(); 
	FTransform* SpaceBasesData = MeshSpaceTransforms.GetData();
	const TArray<FBoneIndexType>& RequiredBoneIndexArray = RequiredBones.GetBoneIndicesArray();

	// First bone is always root bone, and it doesn't have a parent.
	{
		check( RequiredBoneIndexArray[0] == 0 );
		MeshSpaceTransforms[0] = LocalTransforms[0];
	}

	const int32 NumRequiredBones = RequiredBoneIndexArray.Num();
	for(int32 i=1; i<NumRequiredBones; i++)
	{
		const int32 BoneIndex = RequiredBoneIndexArray[i];
		FPlatformMisc::Prefetch(SpaceBasesData + BoneIndex);

		// For all bones below the root, final component-space transform is relative transform * component-space transform of parent.
		const int32 ParentIndex = RequiredBones.GetParentBoneIndex(BoneIndex);
		FPlatformMisc::Prefetch(SpaceBasesData + ParentIndex);

		FTransform::Multiply(SpaceBasesData + BoneIndex, LocalTransformsData + BoneIndex, SpaceBasesData + ParentIndex);

		checkSlow( MeshSpaceTransforms[BoneIndex].IsRotationNormalized() );
		checkSlow( !MeshSpaceTransforms[BoneIndex].ContainsNaN() );
	}
}

/** 
 *	Utility for taking an array of bone indices and ensuring that all parents are present 
 *	(ie. all bones between those in the array and the root are present). 
 *	Note that this must ensure the invariant that parent occur before children in BoneIndices.
 */
void FAnimationRuntime::EnsureParentsPresent(TArray<FBoneIndexType>& BoneIndices, const FReferenceSkeleton& RefSkeleton )
{
	RefSkeleton.EnsureParentsExist(BoneIndices);
}

void FAnimationRuntime::ExcludeBonesWithNoParents(const TArray<int32>& BoneIndices, const FReferenceSkeleton& RefSkeleton, TArray<int32>& FilteredRequiredBones)
{
	// Filter list, we only want bones that have their parents present in this array.
	FilteredRequiredBones.Reset(BoneIndices.Num());

	for (int32 Index=0; Index<BoneIndices.Num(); Index++)
	{
		const int32& BoneIndex = BoneIndices[Index];
		// Always add root bone.
		if( BoneIndex == 0 )
		{
			FilteredRequiredBones.Add(BoneIndex);
		}
		else
		{
			const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(BoneIndex);
			if( FilteredRequiredBones.Contains(ParentBoneIndex) )
			{
				FilteredRequiredBones.Add(BoneIndex);
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "ExcludeBonesWithNoParents: Filtering out bone (%ls) since parent (%ls) is missing", 
					*RefSkeleton.GetBoneName(BoneIndex).ToString(), *RefSkeleton.GetBoneName(ParentBoneIndex).ToString());
			}
		}
	}
}

void FAnimationRuntime::UpdateDesiredBoneWeight(const TArrayView<FPerBoneBlendWeight> SrcBoneBlendWeights, TArrayView<FPerBoneBlendWeight> TargetBoneBlendWeights, const TArrayView<float> BlendWeights)
{
	// in the future, cache this outside
	ensure (TargetBoneBlendWeights.Num() == SrcBoneBlendWeights.Num());

	FMemory::Memzero(TargetBoneBlendWeights.GetData(), TargetBoneBlendWeights.Num() * sizeof(FPerBoneBlendWeight));

	for (int32 BoneIndex = 0; BoneIndex < SrcBoneBlendWeights.Num(); ++BoneIndex)
	{
		const int32 PoseIndex = SrcBoneBlendWeights[BoneIndex].SourceIndex;
		check(PoseIndex < BlendWeights.Num());
		float TargetBlendWeight = BlendWeights[PoseIndex] * SrcBoneBlendWeights[BoneIndex].BlendWeight;
		
		// if relevant, otherwise all initialized as zero
		if (FAnimWeight::IsRelevant(TargetBlendWeight))
		{
			TargetBoneBlendWeights[BoneIndex].SourceIndex = PoseIndex;
			TargetBoneBlendWeights[BoneIndex].BlendWeight = TargetBlendWeight;
		}
	}
}

struct FBlendPosesPerBoneFilterScratchArea : public TThreadSingleton<FBlendPosesPerBoneFilterScratchArea>
{
	using RotationArray = TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex>;
	using ScaleArray = TCustomBoneIndexArray<FVector, FCompactPoseBoneIndex>;

	RotationArray SourceRotations;
	RotationArray BlendRotations;
	RotationArray TargetRotations;

	ScaleArray SourceScales;
	ScaleArray BlendScales;
	ScaleArray TargetScales;

	TArray<float> MaxPoseWeights;
	TArray<const FBlendedCurve*> SourceCurves;
	TArray<float> SourceWeights;
};


// Helper function to get FTransform from a PoseIndex and BoneIndex
extern "C" const uint8* GetTransformFromArray(const uint8 *BlendPoseBase, const int32 PoseIndex, const int32 BoneIndex)
{
	const TConstArrayView<struct FCompactPose>& BlendPoses = *reinterpret_cast<const TConstArrayView<struct FCompactPose>*>(BlendPoseBase);
	const FTransform* BlendPose = &BlendPoses[PoseIndex][FCompactPoseBoneIndex(BoneIndex)];
	return reinterpret_cast<const uint8*>(BlendPose);
}

void FAnimationRuntime::BlendPosesPerBoneFilter(FCompactPose& BasePose,
		const TConstArrayView<FCompactPose> BlendPoses,
		FBlendedCurve& BaseCurve,
		const TConstArrayView<FBlendedCurve> BlendedCurves,
		UE::Anim::FStackAttributeContainer& BaseAttributes,
		const TArrayView<UE::Anim::FStackAttributeContainer> BlendAttributes,
		FAnimationPoseData& OutAnimationPoseData,
		const TArrayView<FPerBoneBlendWeight> BoneBlendWeights,
		EBlendPosesPerBoneFilterFlags BlendFlags,
		enum ECurveBlendOption::Type CurveBlendOption)
{
	SCOPE_CYCLE_COUNTER(STAT_BlendPosesPerBoneFilter);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	// if no blendpose, outpose = basepose
	if (BlendPoses.Num() == 0)
	{
		OutPose = BasePose;
		OutAttributes = BaseAttributes;
		return;
	}

	const int32 NumBones = BasePose.GetNumBones();
	check(BoneBlendWeights.Num() == NumBones);
	check(OutPose.GetNumBones() == NumBones);

	const int32 NumPoses = BlendPoses.Num();
	for (const FPerBoneBlendWeight& PerBoneBlendWeight : BoneBlendWeights)
	{
		check(PerBoneBlendWeight.SourceIndex >= 0);
		check(PerBoneBlendWeight.SourceIndex < NumPoses);
	}

	for (const FCompactPose& BlendPose : BlendPoses)
	{
		check(BlendPose.GetNumBones() == NumBones);
	}

	FBlendPosesPerBoneFilterScratchArea& ScratchArea = FBlendPosesPerBoneFilterScratchArea::Get();
	TArray<float>& MaxPoseWeights = ScratchArea.MaxPoseWeights;
	MaxPoseWeights.Reset();
	MaxPoseWeights.AddZeroed(NumPoses);

	using RotationArray = FBlendPosesPerBoneFilterScratchArea::RotationArray;
	using ScaleArray = FBlendPosesPerBoneFilterScratchArea::ScaleArray;
	RotationArray& SourceRotations = ScratchArea.SourceRotations;
	RotationArray& BlendRotations = ScratchArea.BlendRotations;
	RotationArray& TargetRotations = ScratchArea.TargetRotations;
	ScaleArray& SourceScales = ScratchArea.SourceScales;
	ScaleArray& BlendScales = ScratchArea.BlendScales;
	ScaleArray& TargetScales = ScratchArea.TargetScales;

	const bool bMeshSpaceRotationBlend = EnumHasAnyFlags(BlendFlags, EBlendPosesPerBoneFilterFlags::MeshSpaceRotation);
	const bool bMeshSpaceScaleBlend = EnumHasAnyFlags(BlendFlags, EBlendPosesPerBoneFilterFlags::MeshSpaceScale);
	const bool bRootSpaceRotationBlend = EnumHasAnyFlags(BlendFlags, EBlendPosesPerBoneFilterFlags::RootSpaceRotation);

	if (bRootSpaceRotationBlend || bMeshSpaceRotationBlend)
	{
		SourceRotations.Reset();
		SourceRotations.AddUninitialized(NumBones);
		BlendRotations.Reset();
		BlendRotations.AddUninitialized(NumBones);
		TargetRotations.Reset();
		TargetRotations.AddUninitialized(NumBones);
	}

	if (bMeshSpaceScaleBlend)
	{
		SourceScales.Reset();
		SourceScales.AddUninitialized(NumBones);
		BlendScales.Reset();
		BlendScales.AddUninitialized(NumBones);
		TargetScales.Reset();
		TargetScales.AddUninitialized(NumBones);
	}

	// helpers for mesh space transform accumulation
	auto AccumulateMeshSpaceRotation = [&](int32 PoseIndex, FCompactPoseBoneIndex BoneIndex, const FQuat& ParentSourceRotation, const FQuat& ParentTargetRotation)
	{
		SourceRotations[BoneIndex] = ParentSourceRotation * BasePose[BoneIndex].GetRotation();
		TargetRotations[BoneIndex] = ParentTargetRotation * BlendPoses[PoseIndex][BoneIndex].GetRotation();
	};
	auto AccumulateMeshSpaceScale = [&](int32 PoseIndex, FCompactPoseBoneIndex BoneIndex, const FVector& ParentSourceScale, const FVector& ParentTargetScale)
	{
		SourceScales[BoneIndex] = ParentSourceScale * BasePose[BoneIndex].GetScale3D();
		TargetScales[BoneIndex] = ParentTargetScale * BlendPoses[PoseIndex][BoneIndex].GetScale3D();
	};


	// helpers for mesh space to local space transformation
	auto ConvertMeshToLocalSpaceRotation = [&](FTransform& BlendAtom, FCompactPoseBoneIndex ParentIndex, FCompactPoseBoneIndex BoneIndex)
	{
		// local -> mesh -> local transformations can cause loss of precision for long bone chains, we have to normalize rotation there.
		FQuat LocalBlendQuat = BlendRotations[ParentIndex].Inverse() * BlendRotations[BoneIndex];
		LocalBlendQuat.Normalize();
		BlendAtom.SetRotation(LocalBlendQuat);
	};
	auto ConvertMeshToLocalSpaceScale = [&](FTransform& BlendAtom, FCompactPoseBoneIndex ParentIndex, FCompactPoseBoneIndex BoneIndex)
	{
		FVector ParentScaleInv = FTransform::GetSafeScaleReciprocal(BlendScales[ParentIndex], UE_SMALL_NUMBER);
		FVector LocalBlendScale = ParentScaleInv * BlendScales[BoneIndex];
		BlendAtom.SetScale3D(LocalBlendScale);
	};


	// helpers for mesh space lerping
	auto LerpMeshSpaceRotation = [&](FCompactPoseBoneIndex BoneIndex, float BlendWeight)
	{
		// Fast lerp produces un-normalized quaternions, so we'll re-normalize.
		BlendRotations[BoneIndex] = FQuat::FastLerp(SourceRotations[BoneIndex], TargetRotations[BoneIndex], BlendWeight);
		BlendRotations[BoneIndex].Normalize();
	};
	auto LerpMeshSpaceScale = [&](FCompactPoseBoneIndex BoneIndex, float BlendWeight)
	{
		BlendScales[BoneIndex] = FMath::Lerp(SourceScales[BoneIndex], TargetScales[BoneIndex], BlendWeight);
	};

	const FBoneContainer& BoneContainer = BasePose.GetBoneContainer();

	if (bRootSpaceRotationBlend && bMeshSpaceScaleBlend)
	{
		for (const FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
		{
			const int32 PoseIndex = BoneBlendWeights[BoneIndex.GetInt()].SourceIndex;
			const FCompactPoseBoneIndex ParentIndex = BoneContainer.GetParentBoneIndex(BoneIndex);

			if (ParentIndex != INDEX_NONE)
			{
				AccumulateMeshSpaceRotation(PoseIndex, BoneIndex, SourceRotations[ParentIndex], TargetRotations[ParentIndex]);
				AccumulateMeshSpaceScale(PoseIndex, BoneIndex, SourceScales[ParentIndex], TargetScales[ParentIndex]);
			}
			else
			{
				AccumulateMeshSpaceScale(PoseIndex, BoneIndex, FVector(1.0f), FVector(1.0f));
				// Ignore the root bone for component-space calculation. This forces our rotations to be in root-space.
				SourceRotations[BoneIndex] = FQuat::Identity;
				TargetRotations[BoneIndex] = FQuat::Identity;
			}

			const FTransform& BaseAtom = BasePose[BoneIndex];
			const FTransform& TargetAtom = BlendPoses[PoseIndex][BoneIndex];
			FTransform BlendAtom;

			const float BlendWeight = FMath::Clamp(BoneBlendWeights[BoneIndex.GetInt()].BlendWeight, 0.f, 1.f);
			MaxPoseWeights[PoseIndex] = FMath::Max(MaxPoseWeights[PoseIndex], BlendWeight);

			if (!FAnimWeight::IsRelevant(BlendWeight))
			{
				BlendAtom = BaseAtom;
				BlendRotations[BoneIndex] = SourceRotations[BoneIndex];
				BlendScales[BoneIndex] = SourceScales[BoneIndex];
			}
			else if (FAnimWeight::IsFullWeight(BlendWeight))
			{
				BlendAtom = TargetAtom;
				BlendRotations[BoneIndex] = TargetRotations[BoneIndex];
				BlendScales[BoneIndex] = TargetScales[BoneIndex];
			}
			else
			{
				BlendAtom = BaseAtom;
				BlendAtom.BlendWith(TargetAtom, BlendWeight);
				LerpMeshSpaceRotation(BoneIndex, BlendWeight);
				LerpMeshSpaceScale(BoneIndex, BlendWeight);
			}

			if (ParentIndex != INDEX_NONE)
			{
				ConvertMeshToLocalSpaceRotation(BlendAtom, ParentIndex, BoneIndex);
				ConvertMeshToLocalSpaceScale(BlendAtom, ParentIndex, BoneIndex);
			}

			OutPose[BoneIndex] = BlendAtom;
		}
	}
	else if (bRootSpaceRotationBlend)
	{
		for (const FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
		{
			const int32 PoseIndex = BoneBlendWeights[BoneIndex.GetInt()].SourceIndex;
			const FCompactPoseBoneIndex ParentIndex = BoneContainer.GetParentBoneIndex(BoneIndex);

			if (ParentIndex != INDEX_NONE)
			{
				AccumulateMeshSpaceRotation(PoseIndex, BoneIndex, SourceRotations[ParentIndex], TargetRotations[ParentIndex]);
			}
			else
			{
				// Ignore the root bone for component-space calculation. This forces our rotations to be in root-space.
				SourceRotations[BoneIndex] = FQuat::Identity;
				TargetRotations[BoneIndex] = FQuat::Identity;
			}

			const FTransform& BaseAtom = BasePose[BoneIndex];
			const FTransform& TargetAtom = BlendPoses[PoseIndex][BoneIndex];
			FTransform BlendAtom;

			const float BlendWeight = FMath::Clamp(BoneBlendWeights[BoneIndex.GetInt()].BlendWeight, 0.f, 1.f);
			MaxPoseWeights[PoseIndex] = FMath::Max(MaxPoseWeights[PoseIndex], BlendWeight);

			if (!FAnimWeight::IsRelevant(BlendWeight))
			{
				BlendAtom = BaseAtom;
				BlendRotations[BoneIndex] = SourceRotations[BoneIndex];
			}
			else if (FAnimWeight::IsFullWeight(BlendWeight))
			{
				BlendAtom = TargetAtom;
				BlendRotations[BoneIndex] = TargetRotations[BoneIndex];
			}
			else
			{
				BlendAtom = BaseAtom;
				BlendAtom.BlendWith(TargetAtom, BlendWeight);
				LerpMeshSpaceRotation(BoneIndex, BlendWeight);
			}

			if (ParentIndex != INDEX_NONE)
			{
				ConvertMeshToLocalSpaceRotation(BlendAtom, ParentIndex, BoneIndex);
			}

			OutPose[BoneIndex] = BlendAtom;
		}
	}
	// blend poses with both mesh space rotation and scaling (we assume uniform scale)
	else if (bMeshSpaceRotationBlend && bMeshSpaceScaleBlend)
	{
#if INTEL_ISPC
		if (bAnim_BlendPosesPerBoneFilter_ISPC_Enabled)
		{
			UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
			{
				AutoRTFM::RecordOpenWrite(
					OutPose.GetMutableBones().GetData(),
					OutPose.GetNumBones() * sizeof(FTransform));
				// AutoRTFM doesn't track writes to Source/Target/Blend arrays, since these are part of the Scratch Area;
				// they are not visible to callers, and don't need to persist.
				ispc::BlendPosesPerBoneFilterScaleRotation(
					reinterpret_cast<ispc::FTransform*>(OutPose.GetMutableBones().GetData()),
					reinterpret_cast<const ispc::FTransform*>(BasePose.GetBones().GetData()),
					reinterpret_cast<const uint8*>(&BlendPoses),
					reinterpret_cast<ispc::FVector4*>(SourceRotations.GetData()),
					reinterpret_cast<ispc::FVector*>(SourceScales.GetData()),
					reinterpret_cast<ispc::FVector4*>(TargetRotations.GetData()),
					reinterpret_cast<ispc::FVector*>(TargetScales.GetData()),
					reinterpret_cast<ispc::FVector4*>(BlendRotations.GetData()),
					reinterpret_cast<ispc::FVector*>(BlendScales.GetData()),
					MaxPoseWeights.GetData(),
					reinterpret_cast<const ispc::FPerBoneBlendWeight*>(BoneBlendWeights.GetData()),
					reinterpret_cast<const int32*>(BoneContainer.GetCompactPoseParentBoneArray().GetData()),
					BasePose.GetNumBones());
			};
		}
		else
#endif
		{
			for (const FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
			{
				const int32 PoseIndex = BoneBlendWeights[BoneIndex.GetInt()].SourceIndex;
				const FCompactPoseBoneIndex ParentIndex = BoneContainer.GetParentBoneIndex(BoneIndex);

				if (ParentIndex != INDEX_NONE)
				{
					AccumulateMeshSpaceRotation(PoseIndex, BoneIndex, SourceRotations[ParentIndex], TargetRotations[ParentIndex]);
					AccumulateMeshSpaceScale(PoseIndex, BoneIndex, SourceScales[ParentIndex], TargetScales[ParentIndex]);
				}
				else
				{
					AccumulateMeshSpaceRotation(PoseIndex, BoneIndex, FQuat::Identity, FQuat::Identity);
					AccumulateMeshSpaceScale(PoseIndex, BoneIndex, FVector(1.0f), FVector(1.0f));
				}

				const FTransform& BaseAtom = BasePose[BoneIndex];
				const FTransform& TargetAtom = BlendPoses[PoseIndex][BoneIndex];
				FTransform BlendAtom;

				const float BlendWeight = FMath::Clamp(BoneBlendWeights[BoneIndex.GetInt()].BlendWeight, 0.f, 1.f);
				MaxPoseWeights[PoseIndex] = FMath::Max(MaxPoseWeights[PoseIndex], BlendWeight);

				if (!FAnimWeight::IsRelevant(BlendWeight))
				{
					BlendAtom = BaseAtom;
					BlendRotations[BoneIndex] = SourceRotations[BoneIndex];
					BlendScales[BoneIndex] = SourceScales[BoneIndex];
				}
				else if (FAnimWeight::IsFullWeight(BlendWeight))
				{
					BlendAtom = TargetAtom;
					BlendRotations[BoneIndex] = TargetRotations[BoneIndex];
					BlendScales[BoneIndex] = TargetScales[BoneIndex];
				}
				else
				{
					BlendAtom = BaseAtom;
					BlendAtom.BlendWith(TargetAtom, BlendWeight);
					LerpMeshSpaceRotation(BoneIndex, BlendWeight);
					LerpMeshSpaceScale(BoneIndex, BlendWeight);
				}

				if (ParentIndex != INDEX_NONE)
				{
					ConvertMeshToLocalSpaceRotation(BlendAtom, ParentIndex, BoneIndex);
					ConvertMeshToLocalSpaceScale(BlendAtom, ParentIndex, BoneIndex);
				}

				OutPose[BoneIndex] = BlendAtom;
			}
		}
	}

	// blend poses with mesh space rotation and local scale
	else if (bMeshSpaceRotationBlend)
	{
#if INTEL_ISPC
		if (bAnim_BlendPosesPerBoneFilter_ISPC_Enabled)
		{
			UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
			{
				AutoRTFM::RecordOpenWrite(
					OutPose.GetMutableBones().GetData(),
					OutPose.GetNumBones() * sizeof(FTransform));
				// AutoRTFM doesn't track writes to Source/Target/Blend arrays, since these are part of the Scratch Area;
				// they are not visible to callers, and don't need to persist.
				ispc::BlendPosesPerBoneFilterRotation(
					reinterpret_cast<ispc::FTransform*>(OutPose.GetMutableBones().GetData()),
					reinterpret_cast<const ispc::FTransform*>(BasePose.GetBones().GetData()),
					reinterpret_cast<const uint8*>(&BlendPoses),
					reinterpret_cast<ispc::FVector4*>(SourceRotations.GetData()),
					reinterpret_cast<ispc::FVector4*>(TargetRotations.GetData()),
					reinterpret_cast<ispc::FVector4*>(BlendRotations.GetData()),
					MaxPoseWeights.GetData(),
					reinterpret_cast<const ispc::FPerBoneBlendWeight*>(BoneBlendWeights.GetData()),
					reinterpret_cast<const int32*>(BoneContainer.GetCompactPoseParentBoneArray().GetData()),
					BasePose.GetNumBones());
			};
		}
		else
#endif
		{
			for (const FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
			{
				const int32 PoseIndex = BoneBlendWeights[BoneIndex.GetInt()].SourceIndex;
				const FCompactPoseBoneIndex ParentIndex = BoneContainer.GetParentBoneIndex(BoneIndex);

				if (ParentIndex != INDEX_NONE)
				{
					AccumulateMeshSpaceRotation(PoseIndex, BoneIndex, SourceRotations[ParentIndex], TargetRotations[ParentIndex]);
				}
				else
				{
					AccumulateMeshSpaceRotation(PoseIndex, BoneIndex, FQuat::Identity, FQuat::Identity);
				}

				const FTransform& BaseAtom = BasePose[BoneIndex];
				const FTransform& TargetAtom = BlendPoses[PoseIndex][BoneIndex];
				FTransform BlendAtom;

				const float BlendWeight = FMath::Clamp(BoneBlendWeights[BoneIndex.GetInt()].BlendWeight, 0.f, 1.f);
				MaxPoseWeights[PoseIndex] = FMath::Max(MaxPoseWeights[PoseIndex], BlendWeight);

				if (!FAnimWeight::IsRelevant(BlendWeight))
				{
					BlendAtom = BaseAtom;
					BlendRotations[BoneIndex] = SourceRotations[BoneIndex];
				}
				else if (FAnimWeight::IsFullWeight(BlendWeight))
				{
					BlendAtom = TargetAtom;
					BlendRotations[BoneIndex] = TargetRotations[BoneIndex];
				}
				else
				{
					BlendAtom = BaseAtom;
					BlendAtom.BlendWith(TargetAtom, BlendWeight);
					LerpMeshSpaceRotation(BoneIndex, BlendWeight);
				}

				if (ParentIndex != INDEX_NONE)
				{
					ConvertMeshToLocalSpaceRotation(BlendAtom, ParentIndex, BoneIndex);
				}

				OutPose[BoneIndex] = BlendAtom;
			}
		}
	}

	// blend poses with mesh space scaling (we assume uniform scale) and local rotation
	else if (bMeshSpaceScaleBlend)
	{
#if INTEL_ISPC
		if (bAnim_BlendPosesPerBoneFilter_ISPC_Enabled)
		{
			UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
			{
				AutoRTFM::RecordOpenWrite(
					OutPose.GetMutableBones().GetData(),
					OutPose.GetNumBones() * sizeof(FTransform));
				// AutoRTFM doesn't track writes to Source/Target/Blend arrays, since these are part of the Scratch Area;
				// they are not visible to callers, and don't need to persist.
				ispc::BlendPosesPerBoneFilterScale(
					reinterpret_cast<ispc::FTransform*>(OutPose.GetMutableBones().GetData()),
					reinterpret_cast<const ispc::FTransform*>(BasePose.GetBones().GetData()),
					reinterpret_cast<const uint8*>(&BlendPoses),
					reinterpret_cast<ispc::FVector*>(SourceScales.GetData()),
					reinterpret_cast<ispc::FVector*>(TargetScales.GetData()),
					reinterpret_cast<ispc::FVector*>(BlendScales.GetData()),
					MaxPoseWeights.GetData(),
					reinterpret_cast<const ispc::FPerBoneBlendWeight*>(BoneBlendWeights.GetData()),
					reinterpret_cast<const int32*>(BoneContainer.GetCompactPoseParentBoneArray().GetData()),
					BasePose.GetNumBones());
			};
		}
		else
#endif
		{
			for (const FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
			{
				const int32 PoseIndex = BoneBlendWeights[BoneIndex.GetInt()].SourceIndex;
				const FCompactPoseBoneIndex ParentIndex = BoneContainer.GetParentBoneIndex(BoneIndex);

				if (ParentIndex != INDEX_NONE)
				{
					AccumulateMeshSpaceScale(PoseIndex, BoneIndex, SourceScales[ParentIndex], TargetScales[ParentIndex]);
				}
				else
				{
					AccumulateMeshSpaceScale(PoseIndex, BoneIndex, FVector(1.0f), FVector(1.0f));
				}

				FTransform BaseAtom = BasePose[BoneIndex];
				FTransform TargetAtom = BlendPoses[PoseIndex][BoneIndex];
				FTransform BlendAtom;

				const float BlendWeight = FMath::Clamp(BoneBlendWeights[BoneIndex.GetInt()].BlendWeight, 0.f, 1.f);
				MaxPoseWeights[PoseIndex] = FMath::Max(MaxPoseWeights[PoseIndex], BlendWeight);

				if (!FAnimWeight::IsRelevant(BlendWeight))
				{
					BlendAtom = BaseAtom;
					BlendScales[BoneIndex] = SourceScales[BoneIndex];
				}
				else if (FAnimWeight::IsFullWeight(BlendWeight))
				{
					BlendAtom = TargetAtom;
					BlendScales[BoneIndex] = TargetScales[BoneIndex];
				}
				else
				{
					BlendAtom = BaseAtom;
					BlendAtom.BlendWith(TargetAtom, BlendWeight);
					LerpMeshSpaceScale(BoneIndex, BlendWeight);
				}

				if (ParentIndex != INDEX_NONE)
				{
					ConvertMeshToLocalSpaceScale(BlendAtom, ParentIndex, BoneIndex);
				}

				OutPose[BoneIndex] = BlendAtom;
			}
		}
	}

	// blend poses with local rotation and scaling
	else
	{
#if INTEL_ISPC
		if (bAnim_BlendPosesPerBoneFilter_ISPC_Enabled)
		{
			UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
			{
				AutoRTFM::RecordOpenWrite(
					OutPose.GetMutableBones().GetData(),
					OutPose.GetNumBones() * sizeof(FTransform));
				ispc::BlendPosesPerBoneFilter(
					reinterpret_cast<ispc::FTransform*>(OutPose.GetMutableBones().GetData()),
					reinterpret_cast<const ispc::FTransform*>(BasePose.GetBones().GetData()),
					reinterpret_cast<const uint8*>(&BlendPoses),
					MaxPoseWeights.GetData(),
					reinterpret_cast<const ispc::FPerBoneBlendWeight*>(BoneBlendWeights.GetData()),
					reinterpret_cast<const int32*>(BoneContainer.GetCompactPoseParentBoneArray().GetData()),
					BasePose.GetNumBones());
			};
		}
		else
#endif
		{
			for (const FCompactPoseBoneIndex BoneIndex : BasePose.ForEachBoneIndex())
			{
				const int32 PoseIndex = BoneBlendWeights[BoneIndex.GetInt()].SourceIndex;

				const FTransform& BaseAtom = BasePose[BoneIndex];
				const FTransform& TargetAtom = BlendPoses[PoseIndex][BoneIndex];
				FTransform BlendAtom;

				const float BlendWeight = FMath::Clamp(BoneBlendWeights[BoneIndex.GetInt()].BlendWeight, 0.f, 1.f);
				MaxPoseWeights[PoseIndex] = FMath::Max(MaxPoseWeights[PoseIndex], BlendWeight);

				if (!FAnimWeight::IsRelevant(BlendWeight))
				{
					BlendAtom = BaseAtom;
				}
				else if (FAnimWeight::IsFullWeight(BlendWeight))
				{
					BlendAtom = TargetAtom;
				}
				else
				{
					BlendAtom = BaseAtom;
					BlendAtom.BlendWith(TargetAtom, BlendWeight);
				}

				OutPose[BoneIndex] = BlendAtom;
			}
		}
	}

	// time to blend curves
	// the way we blend curve per bone
	// is to find out max weight per that pose, and then apply that weight to the curve
	{
		TArray<const FBlendedCurve*>& SourceCurves = ScratchArea.SourceCurves;
		TArray<float>& SourceWeights = ScratchArea.SourceWeights;

		SourceCurves.Reset();
		SourceCurves.SetNumUninitialized(NumPoses + 1);
		SourceWeights.Reset();
		SourceWeights.SetNumUninitialized(NumPoses + 1);

		SourceCurves[0] = &BaseCurve;
		SourceWeights[0] = 1.f;

		for (int32 Idx = 0; Idx < NumPoses; ++Idx)
		{
			SourceCurves[Idx + 1] = &BlendedCurves[Idx];
			SourceWeights[Idx + 1] = MaxPoseWeights[Idx];
		}

		BlendCurves(SourceCurves, SourceWeights, OutCurve, CurveBlendOption);
	}

	{
		UE::Anim::Attributes::BlendAttributesPerBoneFilter(BaseAttributes, BlendAttributes, BoneBlendWeights, OutAttributes);
	}
}

void FAnimationRuntime::CreateMaskWeights(TArray<FPerBoneBlendWeight>& BoneBlendWeights, const TArray<FInputBlendPose>& BlendFilters, const USkeleton* Skeleton)
{
	if ( Skeleton )
	{
		const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
		
		const int32 NumBones = RefSkeleton.GetNum();
		BoneBlendWeights.Reset(NumBones);
		BoneBlendWeights.AddZeroed(NumBones);

		// base mask bone
		for (int32 PoseIndex=0; PoseIndex<BlendFilters.Num(); ++PoseIndex)
		{
			const FInputBlendPose& BlendPose = BlendFilters[PoseIndex];

			for (int32 BranchIndex=0; BranchIndex<BlendPose.BranchFilters.Num(); ++BranchIndex)
			{
				const FBranchFilter& BranchFilter = BlendPose.BranchFilters[BranchIndex];
				const int32 MaskBoneIndex = RefSkeleton.FindBoneIndex(BranchFilter.BoneName);

				if (MaskBoneIndex != INDEX_NONE)
				{
					// how much weight increase Per depth
					const float IncreaseWeightPerDepth = (BranchFilter.BlendDepth != 0) ? (1.f/((float)BranchFilter.BlendDepth)) : 1.f;

					// go through skeleton bone hierarchy.
					// Bones are ordered, parents before children. So we can start looking at MaskBoneIndex for children.
					for (int32 BoneIndex = MaskBoneIndex; BoneIndex < NumBones; ++BoneIndex)
					{
						// if Depth == -1, it's not a child
						const int32 Depth = RefSkeleton.GetDepthBetweenBones(BoneIndex, MaskBoneIndex);
						if (Depth != -1)
						{
							// when you write to buffer, you'll need to match with BasePoses BoneIndex
							FPerBoneBlendWeight& BoneBlendWeight = BoneBlendWeights[BoneIndex];

							BoneBlendWeight.SourceIndex = PoseIndex;
							const float BlendIncrease = IncreaseWeightPerDepth * (float)(Depth + 1);
							BoneBlendWeight.BlendWeight = FMath::Clamp<float>(BoneBlendWeight.BlendWeight + BlendIncrease, 0.f, 1.f);
						}
					}
				}
			}
		}
	}
}

void FAnimationRuntime::CreateMaskWeights(TArray<FPerBoneBlendWeight>& BoneBlendWeights, const TArray<class UBlendProfile*>& BlendMasks, const USkeleton* Skeleton)
{
	if (Skeleton)
	{
		const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

		const int32 NumBones = RefSkeleton.GetNum();
		BoneBlendWeights.Reset(NumBones);
		// We only store non-zero weights in blend masks. Initialize all to zero.
		BoneBlendWeights.AddZeroed(NumBones);

		for (int32 MaskIndex = 0; MaskIndex < BlendMasks.Num(); ++MaskIndex)
		{
			const UBlendProfile* BlendMask = BlendMasks[MaskIndex];

			if (!BlendMask || BlendMask->Mode != EBlendProfileMode::BlendMask)
			{
				continue;
			}

			const USkeleton* MaskSkeleton = BlendMask->OwningSkeleton;
			const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(MaskSkeleton, Skeleton);
			for (int32 EntryIndex = 0; EntryIndex < BlendMask->GetNumBlendEntries(); EntryIndex++)
			{
				const int32 BoneIndex = SkeletonRemapping.IsValid() ? SkeletonRemapping.GetTargetSkeletonBoneIndex(BlendMask->ProfileEntries[EntryIndex].BoneReference.BoneIndex) : BlendMask->ProfileEntries[EntryIndex].BoneReference.BoneIndex;
				if (BoneBlendWeights.IsValidIndex(BoneIndex))
				{
					// Match the BoneBlendWeight's input pose with BlendMasks's MaskIndex and use the blend mask's weight
					FPerBoneBlendWeight& BoneBlendWeight = BoneBlendWeights[BoneIndex];

					BoneBlendWeight.SourceIndex = MaskIndex;
					BoneBlendWeight.BlendWeight = BlendMask->ProfileEntries[EntryIndex].BlendScale;
				}
			}
		}
	}
}

void FAnimationRuntime::ConvertCSTransformToBoneSpace(const FTransform& ComponentTransform, FCSPose<FCompactPose>& MeshBases, FTransform& InOutCSBoneTM, FCompactPoseBoneIndex BoneIndex, EBoneControlSpace Space)
{
	switch( Space )
	{
		case BCS_WorldSpace : 
			// world space, so component space * component to world
			InOutCSBoneTM *= ComponentTransform;
			break;

		case BCS_ComponentSpace :
			// Component Space, no change.
			break;

		case BCS_ParentBoneSpace :
			{
				const FCompactPoseBoneIndex ParentIndex = MeshBases.GetPose().GetParentBoneIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(ParentIndex);
					InOutCSBoneTM.SetToRelativeTransform(ParentTM);
				}
			}
			break;

		case BCS_BoneSpace :
			{
				const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(BoneIndex);
				InOutCSBoneTM.SetToRelativeTransform(BoneTM);
			}
			break;

		default:
			UE_LOGF(LogAnimation, Warning, "ConvertCSTransformToBoneSpace: Unknown BoneSpace %d", (int32)Space);
			break;
	}
}

void FAnimationRuntime::ConvertBoneSpaceTransformToCS(const FTransform& ComponentTransform, FCSPose<FCompactPose>& MeshBases, FTransform& InOutBoneSpaceTM, FCompactPoseBoneIndex BoneIndex, EBoneControlSpace Space)
{
	switch( Space )
	{
		case BCS_WorldSpace : 
			InOutBoneSpaceTM.SetToRelativeTransform(ComponentTransform);
			break;

		case BCS_ComponentSpace :
			// Component Space, no change.
			break;

		case BCS_ParentBoneSpace :
			if( BoneIndex != INDEX_NONE )
			{
				const FCompactPoseBoneIndex ParentIndex = MeshBases.GetPose().GetParentBoneIndex(BoneIndex);
				if( ParentIndex != INDEX_NONE )
				{
					const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(ParentIndex);
					InOutBoneSpaceTM *= ParentTM;
				}
			}
			break;

		case BCS_BoneSpace :
			if( BoneIndex != INDEX_NONE )
			{
				const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(BoneIndex);
				InOutBoneSpaceTM *= BoneTM;
			}
			break;

		default:
			UE_LOGF(LogAnimation, Warning, "ConvertBoneSpaceTransformToCS: Unknown BoneSpace %d", (int32)Space);
			break;
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////
/// Pose conversion functions
////////////////////////////////////////////////////////////////////////////////////////////////////

FTransform FAnimationRuntime::GetSpaceTransform(FA2Pose& Pose, int32 Index)
{
	return Pose.Bones[Index];
}

FTransform FAnimationRuntime::GetSpaceTransform(FA2CSPose& Pose, int32 Index)
{
	return Pose.GetComponentSpaceTransform(Index);
}

void FAnimationRuntime::SetSpaceTransform(FA2Pose& Pose, int32 Index, FTransform& NewTransform)
{
	Pose.Bones[Index] = NewTransform;
}

void FAnimationRuntime::SetSpaceTransform(FA2CSPose& Pose, int32 Index, FTransform& NewTransform)
{
	Pose.SetComponentSpaceTransform(Index, NewTransform);
}

void FAnimationRuntime::TickBlendWeight(float DeltaTime, float DesiredWeight, float& Weight, float& BlendTime)
{
	// if it's not same, we'll need to update weight
	if (DesiredWeight != Weight)
	{
		if (BlendTime == 0.f)
		{
			// no blending, just go
			Weight = DesiredWeight;
		}
		else
		{
			float WeightChangePerTime = (DesiredWeight-Weight)/BlendTime;
			Weight += WeightChangePerTime*DeltaTime;

			// going up or down, changes where to clamp to 
			if (WeightChangePerTime > 0.f)
			{
				Weight = FMath::Clamp<float>(Weight, 0.f, DesiredWeight);
			}
			else // if going down
			{
				Weight = FMath::Clamp<float>(Weight, DesiredWeight, 1.f);
			}

			BlendTime-=DeltaTime;
		}
	}
}

#if DO_GUARD_SLOW
// use checkSlow to use this function for debugging
bool FAnimationRuntime::ContainsNaN(TArray<FBoneIndexType>& RequiredBoneIndices, FA2Pose& Pose) 
{
	for (int32 Iter = 0; Iter < RequiredBoneIndices.Num(); ++Iter)
	{
		const int32 BoneIndex = RequiredBoneIndices[Iter];
		if (Pose.Bones[BoneIndex].ContainsNaN())
		{
			return true;
		}
	}

	return false;
}
#endif

FTransform FAnimationRuntime::GetComponentSpaceTransform(const FReferenceSkeleton& RefSkeleton, const TArrayView<const FTransform> &BoneSpaceTransforms, int32 BoneIndex)
{
	if (RefSkeleton.IsValidIndex(BoneIndex))
	{
		// initialize to identity since some of them don't have tracks
		int32 IterBoneIndex = BoneIndex;
		FTransform CompTransform = BoneSpaceTransforms[BoneIndex];

		do
		{
			int32 ParentIndex = RefSkeleton.GetParentIndex(IterBoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				CompTransform = CompTransform * BoneSpaceTransforms[ParentIndex];
			}

			IterBoneIndex = ParentIndex;
		} while (RefSkeleton.IsValidIndex(IterBoneIndex));

		return CompTransform;
	}

	return FTransform::Identity;
}

FTransform FAnimationRuntime::GetComponentSpaceTransformRefPose(const FReferenceSkeleton& RefSkeleton, int32 BoneIndex)
{
	return GetComponentSpaceTransform(RefSkeleton, RefSkeleton.GetRefBonePose(), BoneIndex);
}

const FTransform& FAnimationRuntime::GetComponentSpaceTransformWithCache(const FReferenceSkeleton& RefSkeleton, const TArray<FTransform> &BoneSpaceTransforms, int32 BoneIndex, TArray<FTransform>& CachedTransforms, TArray<bool>& CachedTransformReady)
{
	if (!CachedTransformReady[BoneIndex])
	{
		CachedTransformReady[BoneIndex] = true;
		CachedTransforms[BoneIndex] = FTransform::Identity;

		if (RefSkeleton.IsValidIndex(BoneIndex))
		{
			int32 ParentBoneIndex = RefSkeleton.GetParentIndex(BoneIndex);
			if (ParentBoneIndex != INDEX_NONE)
			{
				const FTransform& ParentComponentSpaceTransform = GetComponentSpaceTransformWithCache(RefSkeleton, BoneSpaceTransforms, ParentBoneIndex, CachedTransforms, CachedTransformReady);
				CachedTransforms[BoneIndex] = BoneSpaceTransforms[BoneIndex] * ParentComponentSpaceTransform;
			}
			else
			{
				CachedTransforms[BoneIndex] = BoneSpaceTransforms[BoneIndex];
			}
		}
	}
	return CachedTransforms[BoneIndex];
}

void FAnimationRuntime::FillUpComponentSpaceTransforms(const FReferenceSkeleton& RefSkeleton, const TArrayView<const FTransform> &BoneSpaceTransforms, TArray<FTransform> &ComponentSpaceTransforms)
{
	ComponentSpaceTransforms.Reset(BoneSpaceTransforms.Num());
	ComponentSpaceTransforms.AddUninitialized(BoneSpaceTransforms.Num());

	// initialize to identity since some of them don't have tracks
	for (int Index = 0; Index < ComponentSpaceTransforms.Num(); ++Index)
	{
		int32 ParentIndex = RefSkeleton.GetParentIndex(Index);
		if (ParentIndex != INDEX_NONE)
		{
			ComponentSpaceTransforms[Index] = BoneSpaceTransforms[Index] * ComponentSpaceTransforms[ParentIndex];
		}
		else
		{
			ComponentSpaceTransforms[Index] = BoneSpaceTransforms[Index];
		}
	}
}

void FAnimationRuntime::MakeSkeletonRefPoseFromMesh(const USkeletalMesh* InMesh, const USkeleton* InSkeleton, TArray<FTransform>& OutBoneBuffer)
{
	check(InMesh && InSkeleton);

	const TArray<FTransform>& MeshRefPose = InMesh->GetRefSkeleton().GetRefBonePose();
	const TArray<FTransform>& SkeletonRefPose = InSkeleton->GetReferenceSkeleton().GetRefBonePose();
	const TArray<FMeshBoneInfo> & SkeletonBoneInfo = InSkeleton->GetReferenceSkeleton().GetRefBoneInfo();

	OutBoneBuffer.Reset(SkeletonRefPose.Num());
	OutBoneBuffer.AddUninitialized(SkeletonRefPose.Num());

	for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < SkeletonRefPose.Num(); ++SkeletonBoneIndex)
	{
		FName SkeletonBoneName = SkeletonBoneInfo[SkeletonBoneIndex].Name;
		int32 MeshBoneIndex = InMesh->GetRefSkeleton().FindBoneIndex(SkeletonBoneName);
		if (MeshBoneIndex != INDEX_NONE)
		{
			OutBoneBuffer[SkeletonBoneIndex] = MeshRefPose[MeshBoneIndex];
		}
		else
		{
			OutBoneBuffer[SkeletonBoneIndex] = FTransform::Identity;
		}
	}
}

#if WITH_EDITOR
void FAnimationRuntime::FillUpComponentSpaceTransformsRefPose(const USkeleton* Skeleton, TArray<FTransform> &ComponentSpaceTransforms)
{
	check(Skeleton);

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	const TArray<FTransform>& ReferencePose = RefSkeleton.GetRefBonePose();
	FillUpComponentSpaceTransforms(RefSkeleton, ReferencePose, ComponentSpaceTransforms);
}
#endif

void FAnimationRuntime::AppendActiveMorphTargets(
	const USkeletalMesh* InSkeletalMesh,
	const TMap<FName, float>& MorphCurveAnims,
	FMorphTargetWeightMap& InOutActiveMorphTargets,
	TArray<float>& InOutMorphTargetWeights
	)
{
	if (!InSkeletalMesh)
	{
		return;
	}

	// ensure the buffer fits the size

	// @note that this only adds zero buffer if it doesn't have enough buffer with the correct size and that is intended
	// there is three places to resize this buffer
	//
	// one is init anim, where we initialize the buffer first time. We need this so that if you don't call Tick, it can have buffer assigned for renderer to get
	// second is tick component, where we make sure the buffer size is correct. We need that so that if you don't have animation or your morphtarget buffer size changes, we want to make sure that buffer is set correctly
	// third is this place where the buffer really matters for game thread, we need to resize if needed in case morphtarget is deleted or added. 
	// the reason you need this is because some other places calling append buffer without going through proper tick component - for example, calling TickAnimation directly
	//
	// if somehow it gets rendered without going through these places, there will be crash. Renderer expect the buffer size being same. 

	if(MorphCurveAnims.IsEmpty())
	{
		return;
	}
	
	const int32 NumMorphTargets = InSkeletalMesh->GetMorphTargets().Num();
	InOutMorphTargetWeights.SetNumZeroed(NumMorphTargets);

	if(NumMorphTargets == 0)
	{
		return;
	}
	
	const float MorphTargetMaxBlendWeight = UE::SkeletalRender::Settings::GetMorphTargetMaxBlendWeight();

	// Then go over the CurveKeys finding morph targets by name
	for(const TPair<FName, float>& MorphCurveAnim : MorphCurveAnims)
	{
		const FName& CurveName = MorphCurveAnim.Key;
		const float Weight = MorphCurveAnim.Value;

		// Find morph reference
		int32 SkeletalMorphIndex = INDEX_NONE;
		const UMorphTarget* Target = InSkeletalMesh->FindMorphTargetAndIndex(CurveName, SkeletalMorphIndex);
		if (Target != nullptr)
		{
			// See if this morph target already has an entry
			const int32* FoundMorphIndex = InOutActiveMorphTargets.Find(Target);
			
			// If it has a valid weight
			if (FMath::Abs(Weight) > MinMorphTargetBlendWeight)
			{
				const float ClampedWeight = FMath::Clamp(Weight, -MorphTargetMaxBlendWeight, MorphTargetMaxBlendWeight);
				// If not, add it
				if (FoundMorphIndex == nullptr)
				{
					InOutActiveMorphTargets.Add(Target, SkeletalMorphIndex);
					InOutMorphTargetWeights[SkeletalMorphIndex] = ClampedWeight;
				}
				else
				{
					// If it does, use the max weight
					check(SkeletalMorphIndex == *FoundMorphIndex);
					InOutMorphTargetWeights[SkeletalMorphIndex] = ClampedWeight;
				}
			}
			else if (FoundMorphIndex != nullptr)
			{
				// The target weight is below the minimum. Force to zero.
				check(SkeletalMorphIndex == *FoundMorphIndex);
				InOutMorphTargetWeights[SkeletalMorphIndex] = 0.f;
			}
		}
	}
}

int32 FAnimationRuntime::GetStringDistance(const FString& First, const FString& Second) 
{
	// Finds the distance between strings, where the distance is the number of operations we would need
	// to perform on First to match Second.
	// Operations are: Adding a character, Removing a character, changing a character.

	const int32 FirstLength = First.Len();
	const int32 SecondLength = Second.Len();

	// Already matching
	if (First == Second)
	{
		return 0;
	}

	// No first string, so we need to add SecondLength characters to match
	if (FirstLength == 0)
	{
		return SecondLength;
	}

	// No Second string, so we need to add FirstLength characters to match
	if (SecondLength == 0)
	{
		return FirstLength;
	}

	TArray<int32> PrevRow;
	TArray<int32> NextRow;
	PrevRow.AddZeroed(SecondLength + 1);
	NextRow.AddZeroed(SecondLength + 1);

	// Initialise prev row to num characters we need to remove from Second
	for (int32 I = 0; I < PrevRow.Num(); ++I)
	{
		PrevRow[I] = I;
	}

	for (int32 I = 0; I < FirstLength; ++I)
	{
		// Calculate current row
		NextRow[0] = I + 1;

		for (int32 J = 0; J < SecondLength; ++J)
		{
			int32 Indicator = (First[I] == Second[J]) ? 0 : 1;
			NextRow[J + 1] = FMath::Min3(NextRow[J] + 1, PrevRow[J + 1] + 1, PrevRow[J] + Indicator);
		}

		// Copy back
		PrevRow = NextRow;
	}

	return NextRow[SecondLength];
}

void FAnimationRuntime::RetargetBoneTransform(const USkeleton* SourceSkeleton, const FName& RetargetSource, FTransform& BoneTransform, const int32 SkeletonBoneIndex, const FCompactPoseBoneIndex& BoneIndex, const FBoneContainer& RequiredBones, const bool bIsBakedAdditive)
{
	if (SourceSkeleton)
	{
		const TArray<FTransform>& RetargetTransforms = SourceSkeleton->GetRefLocalPoses(RetargetSource);
		RetargetBoneTransform(SourceSkeleton, RetargetSource, RetargetTransforms, BoneTransform, SkeletonBoneIndex, BoneIndex, RequiredBones, bIsBakedAdditive);
	}
}

EBoneTranslationRetargetingMode::Type FAnimationRuntime::GetBoneTranslationRetargetingMode(const bool bUseSourceRetargetModes, const int32 SourceSkeletonBoneIndex, const int32 TargetSkeletonBoneIndex, const USkeleton* SourceSkeleton, const USkeleton* TargetSkeleton, const bool bDisableRetargeting)
{
	if (bUseSourceRetargetModes)
	{
		return (SourceSkeletonBoneIndex != INDEX_NONE) 
			? SourceSkeleton->GetBoneTranslationRetargetingMode(SourceSkeletonBoneIndex, bDisableRetargeting) 
			: TargetSkeleton->GetBoneTranslationRetargetingMode(TargetSkeletonBoneIndex, bDisableRetargeting);
	}
	else
	{
		return TargetSkeleton->GetBoneTranslationRetargetingMode(TargetSkeletonBoneIndex, bDisableRetargeting);
	}
}

void FAnimationRuntime::RetargetBoneTransform(const USkeleton* SourceSkeleton, const FName& SourceName, const TArray<FTransform>& RetargetTransforms, FTransform& BoneTransform, const int32 SkeletonBoneIndex, const FCompactPoseBoneIndex& BoneIndex, const FBoneContainer& RequiredBones, const bool bIsBakedAdditive)
{
	FBoneContainerRetargetBoneTransformContext Context(RequiredBones);
	RetargetBoneTransform(SourceSkeleton, SourceName, RetargetTransforms, BoneTransform, SkeletonBoneIndex, BoneIndex, &Context, bIsBakedAdditive);
}

void FAnimationRuntime::RetargetBoneTransform(const USkeleton* SourceSkeleton, const FName& SourceName, const TArray<FTransform>& RetargetTransforms, FTransform& BoneTransform, const int32 SkeletonBoneIndex, const FCompactPoseBoneIndex& BoneIndex, const FAnimationRuntimeRetargetBoneTransformContext* InContext, const bool bIsBakedAdditive)
{
	check(InContext);
	check(!RetargetTransforms.IsEmpty());
	if (SourceSkeleton)
	{
		// Retrieve skeleton, even if it is unreachable (but not GC-ed yet)
		constexpr bool bEvenIfUnreachable = true; 
		const USkeleton* TargetSkeleton = InContext->GetSkeleton(bEvenIfUnreachable);
		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);

		const int32 TargetSkeletonBoneIndex = InContext->GetSkeletonBoneIndex(BoneIndex.GetInt());
		int32 SourceSkeletonBoneIndex = SkeletonBoneIndex;

		// Apply compatible skeleton remapping if required
		if (SkeletonRemapping.IsValid() && SkeletonRemapping.RequiresReferencePoseRetarget())
		{
			const int32 SourceIndex = SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex);
			if (SourceIndex != INDEX_NONE)
			{
				SourceSkeletonBoneIndex = SourceIndex;
				BoneTransform = SkeletonRemapping.RetargetBoneTransformToTargetSkeleton(TargetSkeletonBoneIndex, BoneTransform);
			}
		}

		// Check what retarget mode to use for the translational retargeting for this specific bone.
		const bool bUseSourceRetargetModes = TargetSkeleton->GetUseRetargetModesFromCompatibleSkeleton();
		const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
			bUseSourceRetargetModes,
			SourceSkeletonBoneIndex,
			TargetSkeletonBoneIndex,
			SourceSkeleton,
			TargetSkeleton,
			InContext->IsRetargetingDisabled());
			
		switch (RetargetMode)
		{
			case EBoneTranslationRetargetingMode::AnimationScaled:
			{
				// @todo - precache that in FBoneContainer when we have SkeletonIndex->TrackIndex mapping. So we can just apply scale right away.
				if (RetargetTransforms.IsValidIndex(SourceSkeletonBoneIndex))
				{
					const TArray<FTransform>& SkeletonRefPoseArray = RetargetTransforms;
					const float SourceTranslationLength = SkeletonRefPoseArray[SourceSkeletonBoneIndex].GetTranslation().Size();
					if (SourceTranslationLength > UE_KINDA_SMALL_NUMBER)
					{
						const float TargetTranslationLength = InContext->GetRefPoseTransform(BoneIndex.GetInt()).GetTranslation().Size();
						BoneTransform.ScaleTranslation(TargetTranslationLength / SourceTranslationLength);
					}
				}
				break;
			}

			case EBoneTranslationRetargetingMode::Skeleton:
			{
				BoneTransform.SetTranslation(bIsBakedAdditive ? FVector::ZeroVector : InContext->GetRefPoseTransform(BoneIndex.GetInt()).GetTranslation());
				break;
			}

			case EBoneTranslationRetargetingMode::AnimationRelative:
			{
				// With baked additive animations, Animation Relative delta gets canceled out, so we can skip it.
				// (A1 + Rel) - (A2 + Rel) = A1 - A2.
				if (!bIsBakedAdditive)
				{
					const TArray<FTransform>& AuthoredOnRefSkeleton = RetargetTransforms;
					const FTransform& RefPoseTransform = InContext->GetRefPoseTransform(BoneIndex.GetInt());

					// Remap the base pose onto the target skeleton so that we are working entirely in target space
					if(AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
					{
						FTransform BaseTransform = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex];
						if (SkeletonRemapping.RequiresReferencePoseRetarget())
						{
							BaseTransform = SkeletonRemapping.RetargetBoneTransformToTargetSkeleton(TargetSkeletonBoneIndex, BaseTransform);
						}

						// Apply the retargeting as if it were an additive difference between the current skeleton and the retarget skeleton. 
						BoneTransform.SetRotation(BoneTransform.GetRotation() * BaseTransform.GetRotation().Inverse() * RefPoseTransform.GetRotation());
						BoneTransform.SetTranslation(BoneTransform.GetTranslation() + (RefPoseTransform.GetTranslation() - BaseTransform.GetTranslation()));
						BoneTransform.SetScale3D(BoneTransform.GetScale3D() * (RefPoseTransform.GetScale3D() * BaseTransform.GetSafeScaleReciprocal(BaseTransform.GetScale3D())));
						BoneTransform.NormalizeRotation();
					}
					else
					{
						static bool bLogOnce = true;
						if (bLogOnce)	// If there are several assets broken, we will skip all but first. This is done to avoid log spam.
						{
							UE_LOGF(LogAnimation, Error, "RetargetBoneTransform: Trying to retarget an additive bone (%ls) from SourceName (%ls) with Source Skeleton (%ls) and Target Skeleton (%ls) that does not exist at RetargetTransforms Array. Please check the animation and its Base Pose Aimation and re-save them.",
								*SourceSkeleton->GetReferenceSkeleton().GetBoneName(SourceSkeletonBoneIndex).ToString()
								, *SourceName.ToString()
								, *SourceSkeleton->GetFullName()
								, *TargetSkeleton->GetFullName());

							bLogOnce = false;
						}
					}
				}
				break;
			}

			case EBoneTranslationRetargetingMode::OrientAndScale:
			{
				if (!bIsBakedAdditive)
				{
					const FRetargetSourceCachedData& RetargetSourceCachedData = InContext->GetRetargetSourceCachedData(SourceName, SkeletonRemapping, RetargetTransforms);
					const TArray<FOrientAndScaleRetargetingCachedData>& OrientAndScaleDataArray = RetargetSourceCachedData.OrientAndScaleData;
					const TArray<int32>& CompactPoseIndexToOrientAndScaleIndex = RetargetSourceCachedData.CompactPoseIndexToOrientAndScaleIndex;

					// If we have any cached retargeting data.
					if ((OrientAndScaleDataArray.Num() > 0) && (CompactPoseIndexToOrientAndScaleIndex.Num() == InContext->GetNumBonesInContext()))
					{
						const int32 OrientAndScaleIndex = CompactPoseIndexToOrientAndScaleIndex[BoneIndex.GetInt()];
						if (OrientAndScaleIndex != INDEX_NONE)
						{
							const FOrientAndScaleRetargetingCachedData& OrientAndScaleData = OrientAndScaleDataArray[OrientAndScaleIndex];
							const FVector AnimatedTranslation = BoneTransform.GetTranslation();

							// If Translation is not animated, we can just copy the TargetTranslation. No retargeting needs to be done.
							const FVector NewTranslation = (AnimatedTranslation - OrientAndScaleData.SourceTranslation).IsNearlyZero(BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION) ?
								OrientAndScaleData.TargetTranslation :
								OrientAndScaleData.TranslationDeltaOrient.RotateVector(AnimatedTranslation) * OrientAndScaleData.TranslationScale;

							BoneTransform.SetTranslation(NewTranslation);
						}
					}
				}
				break;
			}
		}
	}
}
/////////////////////////////////////////////////////////////////////////////////////////
// FA2CSPose
/////////////////////////////////////////////////////////////////////////////////////////

/** constructor - needs LocalPoses **/
void FA2CSPose::AllocateLocalPoses(const FBoneContainer& InBoneContainer, const FA2Pose& LocalPose)
{
	AllocateLocalPoses(InBoneContainer, LocalPose.Bones);
}

void FA2CSPose::AllocateLocalPoses(const FBoneContainer& InBoneContainer, const FTransformArrayA2& LocalBones)
{
	check( InBoneContainer.IsValid() );
	BoneContainer = &InBoneContainer;

	Bones = LocalBones;
	ComponentSpaceFlags.Init(0, Bones.Num());

	// root is same, so set root first
	check(ComponentSpaceFlags.Num() > 0);
	ComponentSpaceFlags[0] = 1;
}

bool FA2CSPose::IsValid() const
{
	return (BoneContainer && BoneContainer->IsValid());
}

int32 FA2CSPose::GetParentBoneIndex(const int32 BoneIndex) const
{
	checkSlow( IsValid() );
	return BoneContainer->GetParentBoneIndex(BoneIndex);
}

/** Do not access Bones array directly but via this 
 * This will fill up gradually mesh space bases 
 */
FTransform FA2CSPose::GetComponentSpaceTransform(int32 BoneIndex)
{
	check(Bones.IsValidIndex(BoneIndex));

	// if not evaluate, calculate it
	if( ComponentSpaceFlags[BoneIndex] == 0 )
	{
		CalculateComponentSpaceTransform(BoneIndex);
	}

	return Bones[BoneIndex];
}

void FA2CSPose::SetComponentSpaceTransform(int32 BoneIndex, const FTransform& NewTransform)
{
	check (Bones.IsValidIndex(BoneIndex));

	// this one forcefully sets component space transform
	Bones[BoneIndex] = NewTransform;
	ComponentSpaceFlags[BoneIndex] = 1;
}

/**
 * Convert Bone to Local Space.
 */
void FA2CSPose::ConvertBoneToLocalSpace(int32 BoneIndex)
{
	checkSlow( IsValid() );

	// If BoneTransform is in Component Space, then convert it.
	// Never convert Root to Local Space.
	if( BoneIndex > 0 && ComponentSpaceFlags[BoneIndex] == 1 )
	{
		const int32 ParentIndex = BoneContainer->GetParentBoneIndex(BoneIndex);

		// Verify that our Parent is also in Component Space. That should always be the case.
		check( ComponentSpaceFlags[ParentIndex] == 1 );

		// Convert to local space.
		Bones[BoneIndex].SetToRelativeTransform( Bones[ParentIndex] );
		ComponentSpaceFlags[BoneIndex] = 0;
	}
}

/** 
 * Do not access Bones array directly but via this 
 * This will fill up gradually mesh space bases 
 */
FTransform FA2CSPose::GetLocalSpaceTransform(int32 BoneIndex)
{
	check( Bones.IsValidIndex(BoneIndex) );
	checkSlow( IsValid() );

	// if evaluated, calculate it
	if( ComponentSpaceFlags[BoneIndex] )
	{
		const int32 ParentIndex = BoneContainer->GetParentBoneIndex(BoneIndex);
		if (ParentIndex != INDEX_NONE)
		{
			const FTransform ParentTransform = GetComponentSpaceTransform(ParentIndex);
			const FTransform& BoneTransform = Bones[BoneIndex];
			// calculate local space
			return BoneTransform.GetRelativeTransform(ParentTransform);
		}
	}

	return Bones[BoneIndex];
}

void FA2CSPose::SetLocalSpaceTransform(int32 BoneIndex, const FTransform& NewTransform)
{
	check (Bones.IsValidIndex(BoneIndex));

	// this one forcefully sets component space transform
	Bones[BoneIndex] = NewTransform;
	ComponentSpaceFlags[BoneIndex] = 0;
}

/** Calculate all transform till parent **/
void FA2CSPose::CalculateComponentSpaceTransform(int32 BoneIndex)
{
	check( ComponentSpaceFlags[BoneIndex] == 0 );
	checkSlow( IsValid() );

	// root is already verified, so root should not come here
	// check AllocateLocalPoses
	const int32 ParentIndex = BoneContainer->GetParentBoneIndex(BoneIndex);

	// if Parent already has been calculated, use it
	if( ComponentSpaceFlags[ParentIndex] == 0 )
	{
		// if Parent hasn't been calculated, also calculate parents
		CalculateComponentSpaceTransform(ParentIndex);
	}

	// current Bones(Index) should contain LocalPoses.
	Bones[BoneIndex] = Bones[BoneIndex] * Bones[ParentIndex];
	Bones[BoneIndex].NormalizeRotation();
	ComponentSpaceFlags[BoneIndex] = 1;
}

void FA2CSPose::ConvertToLocalPoses(FA2Pose& LocalPoses)  const
{
	checkSlow(IsValid());
	LocalPoses.Bones = Bones;

	// now we need to convert back to local bases
	// only convert back that has been converted to mesh base
	// if it was local base, and if it hasn't been modified
	// that is still okay even if parent is changed, 
	// that doesn't mean this local has to change
	// go from child to parent since I need parent inverse to go back to local
	// root is same, so no need to do Index == 0
	for(int32 BoneIndex=ComponentSpaceFlags.Num()-1; BoneIndex>0; --BoneIndex)
	{
		// root is already verified, so root should not come here
		// check AllocateLocalPoses
		const int32 ParentIndex = BoneContainer->GetParentBoneIndex(BoneIndex);

		// convert back 
		if( ComponentSpaceFlags[BoneIndex] )
		{
			LocalPoses.Bones[BoneIndex].SetToRelativeTransform( LocalPoses.Bones[ParentIndex] );
			LocalPoses.Bones[BoneIndex].NormalizeRotation();
		}
	}
}

int32 IInterpolationIndexProvider::GetPerBoneInterpolationIndex(int32 BoneIndex, const FBoneContainer& RequiredBones, const FPerBoneInterpolationData* Data) const
{
	return GetPerBoneInterpolationIndex(RequiredBones.MakeCompactPoseIndex(FMeshPoseBoneIndex(BoneIndex)), RequiredBones, Data);
}
