// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Matrix.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"
#include "Math/WideVector.isph"

#define INDEX_NONE -1

static const uniform float DELTA = 0.00001f;
static const uniform float ZERO_ANIMWEIGHT_THRESH = DELTA;
static const uniform struct FVector4 DefaultScale = { {1.f, 1.f, 1.f, 0.f} };

struct FPerBoneBlendWeight
{
	int SourceIndex;
	float BlendWeight;
};

export void BlendTransformOverwrite(const uniform FTransform SourcePose[],
									uniform FTransform ResultPose[],
									const uniform float BlendWeight,
									const uniform int NumBones)
{
	for(uniform int BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		ResultPose[BoneIndex] = SourcePose[BoneIndex] * BlendWeight;
	}
}

export void BlendTransformAccumulate(const uniform FTransform SourcePose[],
									 uniform FTransform ResultPose[],
									 const uniform float BlendWeight,
									 const uniform int NumBones)
{
	for(uniform int BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const uniform FTransform Source = SourcePose[BoneIndex];
		uniform FTransform Dest = ResultPose[BoneIndex];

		const uniform FVector4 BlendedRotation = Source.Rotation * BlendWeight;
		Dest.Rotation = VectorAccumulateQuaternionShortestPath(Dest.Rotation, BlendedRotation);

		Dest.Translation = VectorMultiplyAdd(Source.Translation, BlendWeight, Dest.Translation);
		Dest.Scale3D = VectorMultiplyAdd(Source.Scale3D, BlendWeight, Dest.Scale3D);

		ResultPose[BoneIndex] = Dest;
	}
}

export void ConvertPoseToAdditive(uniform FTransform TargetPose[],
								  const uniform FTransform BasePose[],
								  const uniform int NumBones)
{
	for(uniform int BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		uniform FTransform& TargetTransform = TargetPose[BoneIndex];
		const uniform FTransform BaseTransform = BasePose[BoneIndex];

		TargetTransform.Rotation = VectorQuaternionMultiply2(TargetTransform.Rotation, QuatInverse(BaseTransform.Rotation));
		TargetTransform.Translation = VectorSubtract(TargetTransform.Translation, BaseTransform.Translation);
		// additive scale considers how much it grow or lower
		// in order to support blending between different additive scale, we save [(target scale)/(source scale) - 1.f], and this can blend with 
		// other delta scale value
		// when we apply to the another scale, we apply scale * (1 + [additive scale])
		TargetTransform.Scale3D = VectorSubtract(VectorMultiply(TargetTransform.Scale3D, TransformGetSafeScaleReciprocal(BaseTransform.Scale3D, SMALL_NUMBER)), Vector1110);
		TargetTransform.Rotation = VectorNormalizeQuaternion(TargetTransform.Rotation);
	}
}


export void ConvertPoseToMeshRotation(uniform FTransform LocalPoses[],
									  const uniform int ParentBones[],
									  const uniform int NumBones)
{
	for(uniform int BoneIndex = 1; BoneIndex < NumBones; ++BoneIndex)
	{
		const uniform int ParentIndex = ParentBones[BoneIndex];
		const uniform FVector4 LocalPoseRotation = LocalPoses[BoneIndex].Rotation;
		const uniform FVector4 LocalParentRotation = LocalPoses[ParentIndex].Rotation;

		const uniform FVector4 MeshSpaceRotation = VectorQuaternionMultiply2(LocalParentRotation, LocalPoseRotation);
		LocalPoses[BoneIndex].Rotation = MeshSpaceRotation;
	}
}

export void ConvertMeshRotationPoseToLocalSpace(uniform FTransform Poses[],
												const uniform int ParentBones[],
												const uniform int NumBones)
{
	for(uniform int BoneIndex = NumBones - 1; BoneIndex > 0; --BoneIndex)
	{
		const uniform int ParentIndex = ParentBones[BoneIndex];
		const uniform FVector4 PoseRotation = Poses[BoneIndex].Rotation;
		const uniform FVector4 ParentRotation = Poses[ParentIndex].Rotation;

		const uniform FVector4 LocalSpaceRotation = VectorQuaternionMultiply2(QuatInverse(ParentRotation), PoseRotation);
		Poses[BoneIndex].Rotation = LocalSpaceRotation;
	}
}

export void AccumulateWithAdditiveScale(uniform FTransform BasePose[],
										const uniform FTransform AdditivePose[],
										const uniform float BlendWeight,
										const uniform int NumBones)
{
	for(uniform int BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const uniform FTransform Atom = AdditivePose[BoneIndex];
		uniform FTransform Base = BasePose[BoneIndex];

		// SourceAtom = Atom * BlendWeight;
		const uniform FVector4 BlendedRotation = Atom.Rotation * BlendWeight;

		// Translation += SourceAtom.Translation;
		// Scale *= SourceAtom.Scale;
		const uniform FVector4 TranslationResult = Base.Translation + (Atom.Translation * BlendWeight);
		const uniform FVector4 Scale3DResult = Base.Scale3D * ((Atom.Scale3D * BlendWeight) + DefaultScale);

		// Rotation = SourceAtom.Rotation * Rotation;
		BasePose[BoneIndex].Rotation = VectorQuaternionMultiply2(BlendedRotation, Base.Rotation);

		BasePose[BoneIndex].Translation = TranslationResult;
		BasePose[BoneIndex].Scale3D = Scale3DResult;
	}
}

static inline uniform bool IsRelevant(const uniform float InWeight)
{
	return (InWeight > ZERO_ANIMWEIGHT_THRESH);
}

static inline uniform bool IsFullWeight(const uniform float InWeight)
{
	return (InWeight >= (1.f - ZERO_ANIMWEIGHT_THRESH));
}

static inline uniform bool IsNearlyEqual(const uniform float A, const uniform float B)
{
	return abs(A - B) <= SMALL_NUMBER;
}

// Set this Transform to the weighted blend of it and the supplied Transform.
static inline void BlendWith(uniform FTransform &Atom, const uniform FTransform& OtherAtom, const uniform float BlendWeight)
{
	if( BlendWeight > ZERO_ANIMWEIGHT_THRESH )
	{
		if( BlendWeight >= 1.f - ZERO_ANIMWEIGHT_THRESH )
		{
			// if blend is all the way for child2, then just copy its bone atoms
			Atom = OtherAtom;
		}
		else 
		{
			const uniform FVector4 VRotation = VectorLerpQuat(Atom.Rotation, OtherAtom.Rotation, BlendWeight);
			
			// ..and renormalize
			Atom.Rotation = VectorNormalizeQuaternion(VRotation);
			
			// Simple linear interpolation for translation and scale.
			Atom.Translation = VectorLerp(Atom.Translation, OtherAtom.Translation, BlendWeight);
			
			Atom.Scale3D = VectorLerp(Atom.Scale3D, OtherAtom.Scale3D, BlendWeight);
		}
	}
}

extern "C" unmasked const uniform unsigned int8 *uniform GetTransformFromArray(const uniform unsigned int8 *uniform BlendPoseBase, const uniform int PoseIndex, const uniform int BoneIndex);

// blend poses with both mesh space rotation and scaling (we assume uniform scale)
export void BlendPosesPerBoneFilterScaleRotation(uniform FTransform OutPose[],
												 const uniform FTransform BasePose[],
												 const uniform unsigned int8 *uniform BlendPoseBase,
												 uniform FVector4 SourceRotations[],
												 uniform FVector SourceScales[],
												 uniform FVector4 TargetRotations[],
												 uniform FVector TargetScales[],
												 uniform FVector4 BlendRotations[],
												 uniform FVector BlendScales[],
												 uniform float MaxPoseWeights[],
												 const uniform FPerBoneBlendWeight BoneBlendWeights[],
												 const uniform int ParentBoneArray[],
												 const uniform int NumBones)
{
	for(uniform int BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const uniform int PoseIndex = BoneBlendWeights[BoneIndex].SourceIndex;
		const uniform int ParentIndex = ParentBoneArray[BoneIndex];

		const uniform FTransform BaseAtom = BasePose[BoneIndex];
		const uniform FTransform TargetAtom = *((const uniform FTransform *uniform)GetTransformFromArray(BlendPoseBase, PoseIndex, BoneIndex));

		uniform FVector4 SourceRotation, TargetRotation;
		uniform FVector SourceScale, TargetScale;
	
		if (ParentIndex != INDEX_NONE)
		{
			SourceRotation = VectorQuaternionMultiply2(SourceRotations[ParentIndex], BaseAtom.Rotation);
			TargetRotation = VectorQuaternionMultiply2(TargetRotations[ParentIndex], TargetAtom.Rotation);
			SourceScale = SourceScales[ParentIndex] * SetVector(BaseAtom.Scale3D);
			TargetScale = TargetScales[ParentIndex] * SetVector(TargetAtom.Scale3D);
		}
		else
		{
			SourceRotation = VectorQuaternionMultiply2(Vector0001, BaseAtom.Rotation);
			TargetRotation = VectorQuaternionMultiply2(Vector0001, TargetAtom.Rotation);
			SourceScale = OneVector * SetVector(BaseAtom.Scale3D);
			TargetScale = OneVector * SetVector(TargetAtom.Scale3D);
		}

		const uniform float BlendWeight = clamp(BoneBlendWeights[BoneIndex].BlendWeight, 0.f, 1.f);
		MaxPoseWeights[PoseIndex] = max(MaxPoseWeights[PoseIndex], BlendWeight);

		uniform FTransform BlendAtom;
		uniform FVector4 BlendRotation;
		uniform FVector BlendScale;
	
		if (!IsRelevant(BlendWeight))
		{
			BlendAtom = BaseAtom;
			BlendRotation = SourceRotation;
			BlendScale = SourceScale;
		}
		else if (IsFullWeight(BlendWeight))
		{
			BlendAtom = TargetAtom;
			BlendRotation = TargetRotation;
			BlendScale = TargetScale;
		}
		else
		{
			BlendAtom = BaseAtom;
			BlendWith(BlendAtom, TargetAtom, BlendWeight);

			BlendRotation = VectorNormalizeQuaternion(QuatFastLerp(SourceRotation, TargetRotation, BlendWeight));

			BlendScale = VectorLerp(SourceScale, TargetScale, BlendWeight);
		}

		SourceRotations[BoneIndex] = SourceRotation;
		TargetRotations[BoneIndex] = TargetRotation;
		SourceScales[BoneIndex] = SourceScale;
		TargetScales[BoneIndex] = TargetScale;
		
		if (ParentIndex != INDEX_NONE)
		{
			// local -> mesh -> local transformations can cause loss of precision for long bone chains, we have to normalize rotation there.
			const uniform FVector4 LocalBlendQuat = VectorQuaternionMultiply2(QuatInverse(BlendRotations[ParentIndex]), BlendRotation);
			BlendAtom.Rotation = VectorNormalizeQuaternion(LocalBlendQuat);

			const uniform FVector ParentScaleInv = TransformGetSafeScaleReciprocal(BlendScales[ParentIndex], SMALL_NUMBER);
			const uniform FVector LocalBlendScale = ParentScaleInv * BlendScale;
			BlendAtom.Scale3D = SetVector4(LocalBlendScale.V[0], LocalBlendScale.V[1], LocalBlendScale.V[2], ZERO);
		}

		BlendRotations[BoneIndex] = BlendRotation;
		BlendScales[BoneIndex] = BlendScale;

		OutPose[BoneIndex] = BlendAtom;
	}
}


// blend poses with both mesh space rotation (we assume uniform scale)
export void BlendPosesPerBoneFilterRotation(uniform FTransform OutPose[],
											const uniform FTransform BasePose[],
											const uniform unsigned int8 *uniform BlendPoseBase,
											uniform FVector4 SourceRotations[],
											uniform FVector4 TargetRotations[],
											uniform FVector4 BlendRotations[],
											uniform float MaxPoseWeights[],
											const uniform FPerBoneBlendWeight BoneBlendWeights[],
											const uniform int ParentBoneArray[],
											const uniform int NumBones)
{
	for(uniform int BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const uniform int PoseIndex = BoneBlendWeights[BoneIndex].SourceIndex;
		const uniform int ParentIndex = ParentBoneArray[BoneIndex];

		const uniform FTransform BaseAtom = BasePose[BoneIndex];
		const uniform FTransform TargetAtom = *((const uniform FTransform *uniform)GetTransformFromArray(BlendPoseBase, PoseIndex, BoneIndex));

		uniform FVector4 SourceRotation, TargetRotation;
	
		if (ParentIndex != INDEX_NONE)
		{
			SourceRotation = VectorQuaternionMultiply2(SourceRotations[ParentIndex], BaseAtom.Rotation);
			TargetRotation = VectorQuaternionMultiply2(TargetRotations[ParentIndex], TargetAtom.Rotation);
		}
		else
		{
			SourceRotation = VectorQuaternionMultiply2(Vector0001, BaseAtom.Rotation);
			TargetRotation = VectorQuaternionMultiply2(Vector0001, TargetAtom.Rotation);
		}

		const uniform float BlendWeight = clamp(BoneBlendWeights[BoneIndex].BlendWeight, 0.f, 1.f);
		MaxPoseWeights[PoseIndex] = max(MaxPoseWeights[PoseIndex], BlendWeight);

		uniform FTransform BlendAtom;
		uniform FVector4 BlendRotation;

		if (!IsRelevant(BlendWeight))
		{
			BlendAtom = BaseAtom;
			BlendRotation = SourceRotation;
		}
		else if (IsFullWeight(BlendWeight))
		{
			BlendAtom = TargetAtom;
			BlendRotation = TargetRotation;
		}
		else
		{
			BlendAtom = BaseAtom;
			BlendWith(BlendAtom, TargetAtom, BlendWeight);

			BlendRotation = VectorNormalizeQuaternion(QuatFastLerp(SourceRotation, TargetRotation, BlendWeight));
		}

		SourceRotations[BoneIndex] = SourceRotation;
		TargetRotations[BoneIndex] = TargetRotation;

		if (ParentIndex != INDEX_NONE)
		{
			// local -> mesh -> local transformations can cause loss of precision for long bone chains, we have to normalize rotation there.
			const uniform FVector4 LocalBlendQuat = VectorQuaternionMultiply2(QuatInverse(BlendRotations[ParentIndex]), BlendRotation);
			BlendAtom.Rotation = VectorNormalizeQuaternion(LocalBlendQuat);
		}

		BlendRotations[BoneIndex] = BlendRotation;

		OutPose[BoneIndex] = BlendAtom;
	}
}

// blend poses with mesh space scaling (we assume uniform scale)
export void BlendPosesPerBoneFilterScale(uniform FTransform OutPose[],
										 const uniform FTransform BasePose[],
										 const uniform unsigned int8 *uniform BlendPoseBase,
										 uniform FVector SourceScales[],
										 uniform FVector TargetScales[],
										 uniform FVector BlendScales[],
										 uniform float MaxPoseWeights[],
										 const uniform FPerBoneBlendWeight BoneBlendWeights[],
										 const uniform int ParentBoneArray[],
										 const uniform int NumBones)
{
	for(uniform int BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const uniform int PoseIndex = BoneBlendWeights[BoneIndex].SourceIndex;
		const uniform int ParentIndex = ParentBoneArray[BoneIndex];

		const uniform FTransform BaseAtom = BasePose[BoneIndex];
		const uniform FTransform TargetAtom = *((const uniform FTransform *uniform)GetTransformFromArray(BlendPoseBase, PoseIndex, BoneIndex));

		uniform FVector SourceScale, TargetScale;
	
		if (ParentIndex != INDEX_NONE)
		{
			SourceScale = SourceScales[ParentIndex] * SetVector(BaseAtom.Scale3D);
			TargetScale = TargetScales[ParentIndex] * SetVector(TargetAtom.Scale3D);
		}
		else
		{
			SourceScale = OneVector * SetVector(BaseAtom.Scale3D);
			TargetScale = OneVector * SetVector(TargetAtom.Scale3D);
		}

		const uniform float BlendWeight = clamp(BoneBlendWeights[BoneIndex].BlendWeight, 0.f, 1.f);
		MaxPoseWeights[PoseIndex] = max(MaxPoseWeights[PoseIndex], BlendWeight);

		uniform FTransform BlendAtom;
		uniform FVector BlendScale;
	
		if (!IsRelevant(BlendWeight))
		{
			BlendAtom = BaseAtom;
			BlendScale = SourceScale;
		}
		else if (IsFullWeight(BlendWeight))
		{
			BlendAtom = TargetAtom;
			BlendScale = TargetScale;
		}
		else
		{
			BlendAtom = BaseAtom;
			BlendWith(BlendAtom, TargetAtom, BlendWeight);

			BlendScale = VectorLerp(SourceScale, TargetScale, BlendWeight);
		}

		SourceScales[BoneIndex] = SourceScale;
		TargetScales[BoneIndex] = TargetScale;
		
		if (ParentIndex != INDEX_NONE)
		{
			const uniform FVector ParentScaleInv = TransformGetSafeScaleReciprocal(BlendScales[ParentIndex], SMALL_NUMBER);
			const uniform FVector LocalBlendScale = ParentScaleInv * BlendScale;
			BlendAtom.Scale3D = SetVector4(LocalBlendScale.V[0], LocalBlendScale.V[1], LocalBlendScale.V[2], ZERO);
		}

		BlendScales[BoneIndex] = BlendScale;

		OutPose[BoneIndex] = BlendAtom;
	}
}

// blend poses in local space
export void BlendPosesPerBoneFilter(uniform FTransform OutPose[],
									const uniform FTransform BasePose[],
									const uniform unsigned int8 *uniform BlendPoseBase,
									uniform float MaxPoseWeights[],
									const uniform FPerBoneBlendWeight BoneBlendWeights[],
									const uniform int ParentBoneArray[],
									const uniform int NumBones)
{
	for(uniform int BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const uniform int PoseIndex = BoneBlendWeights[BoneIndex].SourceIndex;
		const uniform int ParentIndex = ParentBoneArray[BoneIndex];

		const uniform FTransform BaseAtom = BasePose[BoneIndex];
		const uniform FTransform TargetAtom = *((const uniform FTransform *uniform)GetTransformFromArray(BlendPoseBase, PoseIndex, BoneIndex));

		const uniform float BlendWeight = clamp(BoneBlendWeights[BoneIndex].BlendWeight, 0.f, 1.f);
		MaxPoseWeights[PoseIndex] = max(MaxPoseWeights[PoseIndex], BlendWeight);

		uniform FTransform BlendAtom;
	
		if (!IsRelevant(BlendWeight))
		{
			BlendAtom = BaseAtom;
		}
		else if (IsFullWeight(BlendWeight))
		{
			BlendAtom = TargetAtom;
		}
		else
		{
			BlendAtom = BaseAtom;
			BlendWith(BlendAtom, TargetAtom, BlendWeight);
		}

		OutPose[BoneIndex] = BlendAtom;
	}
}

export void BlendCurves(const uniform float SourceCurveWeights[],
						const uniform uint32 SourceValidCurveWeights[],
						uniform float OutCurveWeights[],
						uniform uint32 OutValidCurveWeights[],
						const uniform int CurveLength,
						const uniform int CurveIndex,
						uniform float SourceWeight)
{
	// Note: Run OutCurve.InitFrom(SourceCurves) before this function runs

	if(CurveIndex == 0 && IsNearlyEqual(SourceWeight, 1.0f))
	{
		SourceWeight = 1.0f;
	}

	foreach (CurveID = 0 ... CurveLength)
	{
		// Skip if the source curve is not valid
		const varying int ValidityArrayIndex = CurveID >> 5; // floor(CurveID / 32.0f)
		const varying int ValidityBitPosition = CurveID & 0x1F; // % 32;
		const varying int ValidityBitMask = 0x1 << ValidityBitPosition;

		if ((SourceValidCurveWeights[ValidityArrayIndex] & ValidityBitMask) == 0)
		{
			continue;
		}

		varying float CurveWeight = OutCurveWeights[CurveID];

		if (CurveIndex == 0)
		{
			CurveWeight = SourceCurveWeights[CurveID] * SourceWeight;
		}
		else if (IsRelevant(SourceWeight))
		{
			CurveWeight += SourceCurveWeights[CurveID] * SourceWeight;
		}

		OutCurveWeights[CurveID] = CurveWeight;
	}

	foreach (CurveID = 0 ... ceil(CurveLength / 32.0))
	{
		if (CurveIndex == 0)
		{
			OutValidCurveWeights[CurveID] = SourceValidCurveWeights[CurveID];
		}
		else
		{
			OutValidCurveWeights[CurveID] |= SourceValidCurveWeights[CurveID];
		}
	}
}

export void LerpBoneTransforms(uniform FTransform ATransformData[],
							   const uniform FTransform BTransformData[],
							   const uniform float Alpha,
							   const uniform unsigned int16 RequiredBonesArray[],
							   const uniform int32 NumOfBones)
{
	// Note: Check FAnimWeight::IsFullWeight(Alpha) as well as !FAnimWeight::IsRelevant(Alpha) before this function runs

	const uniform float OneMinusAlpha = 1 - Alpha;

	uniform int NumOfBonesBase = NumOfBones & ~(programCount-1);

	for (uniform int Index = 0; Index < NumOfBonesBase; Index+=programCount/4)
	{
		uniform int BoneIndex[programCount / 4];

		for (uniform int i = 0; i < programCount / 4; i++)
		{
			BoneIndex[i] = RequiredBonesArray[Index + i] * 3;
		}
		uniform int *uniform pBoneIndex = (uniform int* uniform)&BoneIndex;

		// Do rotation work
		uniform WideFVector4 ARotation, BRotation;
		LoadIndexedWideFVector4((uniform FVector4 *uniform)&ARotation, (uniform FVector4 *uniform)&ATransformData[0].Rotation, pBoneIndex);
		LoadIndexedWideFVector4((uniform FVector4 *uniform)&BRotation, (uniform FVector4 *uniform)&BTransformData[0].Rotation, pBoneIndex);

		// AccumulateWithShortestRotation
		ARotation = ARotation * OneMinusAlpha;
		BRotation = BRotation * Alpha;

		ARotation = VectorAccumulateQuaternionShortestPath(ARotation, BRotation);
		ARotation = VectorNormalizeQuaternion(ARotation);

		StoreIndexedWideFVector4((uniform FVector4 *uniform)&ATransformData[0].Rotation, (uniform FVector4 *uniform)&ARotation, pBoneIndex);

		// Do translation work
		uniform WideFVector4 ATranslation, BTranslation;
		LoadIndexedWideFVector4((uniform FVector4 *uniform)&ATranslation, (uniform FVector4 *uniform)&ATransformData[0].Translation, pBoneIndex);
		LoadIndexedWideFVector4((uniform FVector4 *uniform)&BTranslation, (uniform FVector4 *uniform)&BTransformData[0].Translation, pBoneIndex);

		ATranslation = ATranslation * OneMinusAlpha + BTranslation * Alpha;

		StoreIndexedWideFVector4((uniform FVector4 *uniform)&ATransformData[0].Translation, (uniform FVector4 *uniform)&ATranslation, pBoneIndex);

		// Do scale work
		uniform WideFVector4 AScale3D, BScale3D;
		LoadIndexedWideFVector4((uniform FVector4 *uniform)&AScale3D, (uniform FVector4 *uniform)&ATransformData[0].Scale3D, pBoneIndex);
		LoadIndexedWideFVector4((uniform FVector4 *uniform)&BScale3D, (uniform FVector4 *uniform)&BTransformData[0].Scale3D, pBoneIndex);

		AScale3D = AScale3D * OneMinusAlpha + BScale3D * Alpha;

		StoreIndexedWideFVector4((uniform FVector4 *uniform)&ATransformData[0].Scale3D, (uniform FVector4 *uniform)&AScale3D, pBoneIndex);
	}

	for (uniform int Index = NumOfBonesBase; Index < NumOfBones; Index++)
	{
		// Do rotation work
		const uniform int32 BoneIndex = RequiredBonesArray[Index];
		uniform FVector4 ARotation = ATransformData[BoneIndex].Rotation;
		uniform FVector4 BRotation = BTransformData[BoneIndex].Rotation;

		// AccumulateWithShortestRotation
		ARotation = ARotation * OneMinusAlpha;
		BRotation = BRotation * Alpha;

		ARotation = VectorAccumulateQuaternionShortestPath(ARotation, BRotation);
		ATransformData[BoneIndex].Rotation = VectorNormalizeQuaternion(ARotation);

		// Do translation work
		const uniform FVector4 ATranslation = ATransformData[BoneIndex].Translation;
		const uniform FVector4 BTranslation = BTransformData[BoneIndex].Translation;

		ATransformData[BoneIndex].Translation = ATranslation * OneMinusAlpha + BTranslation * Alpha;

		// Do scale work
		const uniform FVector4 AScale3D = ATransformData[BoneIndex].Scale3D;
		const uniform FVector4 BScale3D = BTransformData[BoneIndex].Scale3D;

		ATransformData[BoneIndex].Scale3D = AScale3D * OneMinusAlpha + BScale3D * Alpha;
	}
}
