// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationSequenceCompiler.h"

#include "Animation/AnimCompress.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "ProfilingDebugging/CountersTrace.h"

#if WITH_EDITOR

#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"
#include "ObjectCacheContext.h"
#include "UObject/Package.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "UObject/StrongObjectPtr.h"
#include "Animation/IAnimationSequenceCompiler.h"

#define LOCTEXT_NAMESPACE "AnimSequenceCompilingManager"

namespace UE::Anim
{
	FAnimSequenceCompilingManager& FAnimSequenceCompilingManager::Get()
	{
		static FAnimSequenceCompilingManager Singleton;
		return Singleton;
	}

	void IAnimSequenceCompilingManager::FinishCompilation(TArrayView<UAnimSequence* const> InAnimSequences)
	{
		FAnimSequenceCompilingManager::Get().FinishCompilation(InAnimSequences); 
	}

	FAnimSequenceCompilingManager::FAnimSequenceCompilingManager()
		: Notification(MakeUnique<FAsyncCompilationNotification>(FAnimSequenceCompilingManager::GetAssetNameFormat()))
	{
		PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddRaw(this, &FAnimSequenceCompilingManager::OnPostReachabilityAnalysis);
	}
	
	TRACE_DECLARE_INT_COUNTER(QueuedAnimationSequenceCompilation, TEXT("AsyncCompilation/QueuedAnimationSequences"));

	void FAnimSequenceCompilingManager::FinishAllCompilation()
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceCompilingManager::FinishAllCompilation)

		if (GetNumRemainingAssets())
		{
			TArray<UAnimSequence*> PendingAnimationSequences;
			PendingAnimationSequences.Reserve(GetNumRemainingAssets());

			for (TWeakObjectPtr<UAnimSequence>& WeakAnimationSequence : RegisteredAnimSequences)
			{
				if (WeakAnimationSequence.IsValid())
				{
					PendingAnimationSequences.Add(WeakAnimationSequence.Get());
				}
			}

			FinishCompilation(PendingAnimationSequences);
		}
	}

	void FAnimSequenceCompilingManager::Shutdown()
	{
		if (GetNumRemainingAssets())
		{
			check(IsInGameThread());
			TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceCompilingManager::Shutdown);

			TArray<UAnimSequence*> PendingAnimationSequences;
			PendingAnimationSequences.Reserve(GetNumRemainingAssets());

			for (TWeakObjectPtr<UAnimSequence>& WeakAnimationSequence : RegisteredAnimSequences)
			{
				if (WeakAnimationSequence.IsValid())
				{
					UAnimSequence* AnimSequence = WeakAnimationSequence.Get();
					if (!AnimSequence->TryCancelAsyncTasks())
					{
						PendingAnimationSequences.Add(AnimSequence);
					}
				}
			}

			FinishCompilation(PendingAnimationSequences);
		}
		
		FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostReachabilityAnalysisHandle);
	}

	FName FAnimSequenceCompilingManager::GetStaticAssetTypeName()
	{
		return TEXT("UE-AnimationSequence");
	}

	FName FAnimSequenceCompilingManager::GetAssetTypeName() const
	{
		return GetStaticAssetTypeName();
	}

	FTextFormat FAnimSequenceCompilingManager::GetAssetNameFormat() const
	{
		return LOCTEXT("AnimationSequenceNameFormat", "{0}|plural(one=Animation Sequence,other=Animation Sequences)");
	}

	TArrayView<FName> FAnimSequenceCompilingManager::GetDependentTypeNames() const
	{
		return TArrayView<FName>();
	}

	int32 FAnimSequenceCompilingManager::GetNumRemainingAssets() const
	{
		return RegisteredAnimSequences.Num();
	}

	FQueuedThreadPool* FAnimSequenceCompilingManager::GetThreadPool() const
	{
		static FQueuedThreadPoolDynamicWrapper* GAnimationSequenceCompressionThreadPool = nullptr;
		if (GAnimationSequenceCompressionThreadPool == nullptr && FAssetCompilingManager::Get().GetThreadPool() != nullptr)
		{
			GAnimationSequenceCompressionThreadPool = new FQueuedThreadPoolDynamicWrapper(FAssetCompilingManager::Get().GetThreadPool(), -1, [](EQueuedWorkPriority) { return EQueuedWorkPriority::Low; });
		}

		return GAnimationSequenceCompressionThreadPool;
	}

	EQueuedWorkPriority FAnimSequenceCompilingManager::GetBasePriority(const UAnimSequence* InAnimSequence) const
	{
		return EQueuedWorkPriority::Normal;
	}

	void FAnimSequenceCompilingManager::AddAnimSequences(TArrayView<UAnimSequence* const> InAnimSequences)
	{
		check(IsInGameThread());

		for (UAnimSequence* AnimSequence : InAnimSequences)
		{
			check(!RegisteredAnimSequences.Find(AnimSequence));
			RegisteredAnimSequences.Emplace(AnimSequence);
		}

		TRACE_COUNTER_SET(QueuedAnimationSequenceCompilation, GetNumRemainingAssets());
	}

	void FAnimSequenceCompilingManager::FinishCompilation(TArrayView<UAnimSequence* const> InAnimSequences)
	{		
		check(IsInGameThread())

		TArray<UAnimSequence*> PendingAnimationSequences;
		PendingAnimationSequences.Reserve(InAnimSequences.Num());

		for (UAnimSequence* AnimSequence : InAnimSequences)
		{
			if (AnimSequence && RegisteredAnimSequences.Contains(AnimSequence))
			{
				PendingAnimationSequences.Emplace(AnimSequence);
			}
		}

		if (PendingAnimationSequences.Num())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimSequenceCompilingManager_FinishCompilation);
			
			class FCompilableAnimationSequence: public AsyncCompilationHelpers::ICompilable
			{
			public:
				FCompilableAnimationSequence(UAnimSequence* InAnimSequence)
					: AnimSequence(InAnimSequence)
				{
				}

				virtual void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority) override
				{
					AnimSequence->Reschedule(InThreadPool, InPriority);
				}

				virtual bool WaitCompletionWithTimeout(float TimeLimitSeconds) override
				{
					return AnimSequence->WaitForAsyncTasks(TimeLimitSeconds);
				}

				TStrongObjectPtr<UAnimSequence> AnimSequence;
				virtual FName GetName() override { return AnimSequence->GetOutermost()->GetFName(); }
			};

			TArray<FCompilableAnimationSequence> CompilableAnimationSequences(PendingAnimationSequences);
			FObjectCacheContextScope ObjectCacheScope;
			AsyncCompilationHelpers::FinishCompilation(
				[&CompilableAnimationSequences](int32 Index) -> AsyncCompilationHelpers::ICompilable& { return CompilableAnimationSequences[Index]; },
				CompilableAnimationSequences.Num(),
				LOCTEXT("AnimationSequences", "Animation Sequences"),
				LogAnimation,
				[this](AsyncCompilationHelpers::ICompilable* Object)
				{
					UAnimSequence* AnimSequence = static_cast<FCompilableAnimationSequence*>(Object)->AnimSequence.Get();
					ApplyCompilation(AnimSequence);
					RegisteredAnimSequences.Remove(AnimSequence);
				}
			);

			PostCompilation(PendingAnimationSequences);
		}
		
	}

	void FAnimSequenceCompilingManager::FinishCompilation(TArrayView<USkeleton* const> InSkeletons)
	{
		TArray<UAnimSequence*> PendingAnimationSequences;	
		for (const TWeakObjectPtr<UAnimSequence>& WeakAnimSequence : RegisteredAnimSequences)
		{
			if (UAnimSequence* AnimSequence = WeakAnimSequence.GetEvenIfUnreachable())
			{
				if (InSkeletons.Contains(AnimSequence->GetSkeleton()))
				{
					PendingAnimationSequences.Add(AnimSequence);
				}
			}
		}

		FinishCompilation(PendingAnimationSequences);
	}

	void FAnimSequenceCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
	{
		FObjectCacheContextScope ObjectCacheScope;
		
		ProcessAnimSequences(bLimitExecutionTime);
		
		UpdateCompilationNotification();
	}

	void FAnimSequenceCompilingManager::ProcessAnimSequences(bool bLimitExecutionTime, int32 MinBatchSize)
	{
		const int32 NumRemaining = GetNumRemainingAssets();
		
		const int32 MaxToProcess = bLimitExecutionTime ? FMath::Max(64, NumRemaining / 10) : INT32_MAX;
		
		FObjectCacheContextScope ObjectCacheScope;
		if (NumRemaining && NumRemaining >= MinBatchSize)
		{
			TSet<UAnimSequence*> SequencesToProcess;
			for (TWeakObjectPtr<UAnimSequence>& AnimSequence : RegisteredAnimSequences)
			{
				if (AnimSequence.IsValid())
				{
					SequencesToProcess.Add(AnimSequence.Get());
				}
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ProcessAnimSequences);

				TSet<TWeakObjectPtr<UAnimSequence>> AnimSequencesToPostpone;
				TArray<UAnimSequence*> ProcessedAnimSequences;
				if (SequencesToProcess.Num())
				{
					for (UAnimSequence* AnimSequence : SequencesToProcess)
					{
						const bool bHasMeshUpdateLeft = ProcessedAnimSequences.Num() <= MaxToProcess;
						if (bHasMeshUpdateLeft && AnimSequence->IsAsyncTaskComplete())
						{
							ApplyCompilation(AnimSequence);
							ProcessedAnimSequences.Add(AnimSequence);
						}
						else
						{
							AnimSequencesToPostpone.Emplace(AnimSequence);
						}
					}
				}

				RegisteredAnimSequences = MoveTemp(AnimSequencesToPostpone);

				PostCompilation(ProcessedAnimSequences);
			}
		}
		
	}

	void FAnimSequenceCompilingManager::PostCompilation(TArrayView<UAnimSequence* const> InAnimSequences)
	{
		if (InAnimSequences.Num())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(OnAssetPostCompileEvent);

			TArray<FAssetCompileData> AssetsData;
			AssetsData.Reserve(InAnimSequences.Num());
			check(IsInGameThread());
			for (UAnimSequence* AnimSequence : InAnimSequences)
			{
				// Do not broadcast an event for unreachable objects
				if (!AnimSequence->IsUnreachable())
				{
					AssetsData.Emplace(AnimSequence);
				}
			}

			if (AssetsData.Num())
			{
				FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast(AssetsData);
			}
		}
	}

	void FAnimSequenceCompilingManager::ApplyCompilation(UAnimSequence* InAnimSequence)
	{
		// If AsyncTask is null here, the task got canceled so we don't need to do anything
		if (!IsEngineExitRequested())
		{
			check(IsInGameThread());
			TRACE_CPUPROFILER_EVENT_SCOPE(PostCompilation);
			if (InAnimSequence && !InAnimSequence->IsUnreachable() && InAnimSequence->GetSkeleton() && !InAnimSequence->GetSkeleton()->IsUnreachable())
			{
				InAnimSequence->FinishAsyncTasks();
			}
			else if (InAnimSequence)
			{				
				UE_LOGF(LogAnimation, Display, "Failed to finish Async Animation Compression for %ls, as it is either unreachable (%i) has a null skeleton (%i) or its skeleton (%ls) is unreachable (%i)", *InAnimSequence->GetName(),
					InAnimSequence->IsUnreachable(), InAnimSequence->GetSkeleton() == nullptr,  InAnimSequence->GetSkeleton() == nullptr ? TEXT("null") :  *InAnimSequence->GetName(), InAnimSequence->GetSkeleton()->IsUnreachable());
			}
		}
	}
	void FAnimSequenceCompilingManager::UpdateCompilationNotification()
	{
		TRACE_COUNTER_SET(QueuedAnimationSequenceCompilation, GetNumRemainingAssets());
		Notification->Update(GetNumRemainingAssets());
	}

	void FAnimSequenceCompilingManager::OnPostReachabilityAnalysis()
	{
		if (GetNumRemainingAssets())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceCompilingManager::CancelUnreachableMeshes);

			TArray<UAnimSequence*> PendingAnimationSequences;
			PendingAnimationSequences.Reserve(GetNumRemainingAssets());

			for (auto Iterator = RegisteredAnimSequences.CreateIterator(); Iterator; ++Iterator)
			{
				if (UAnimSequence* AnimSequence = Iterator->GetEvenIfUnreachable())
				{
					const bool bAnimSequenceUnreachable = AnimSequence->IsUnreachable();
					const bool bNullSkeleton = AnimSequence->GetSkeleton() == nullptr;
					const bool bSkeletonUnreachable = bNullSkeleton ? true : AnimSequence->GetSkeleton()->IsUnreachable();
					
					if(bAnimSequenceUnreachable||bNullSkeleton||bSkeletonUnreachable)
					{
						if (AnimSequence->TryCancelAsyncTasks())
						{
							Iterator.RemoveCurrent();
						}
						else
						{
							PendingAnimationSequences.Add(AnimSequence);
						}
						
						UE_LOGF(LogAnimation, Display, "Cancelling Async Animation Compression for %ls, as it is either unreachable (%i) has a null skeleton (%i) or its skeleton (%ls) is unreachable (%i)", *AnimSequence->GetName(),
							AnimSequence->IsUnreachable(), AnimSequence->GetSkeleton() == nullptr,  AnimSequence->GetSkeleton() == nullptr ? TEXT("null") :  *AnimSequence->GetName(), AnimSequence->GetSkeleton()->IsUnreachable());
					}
				}
			}

			FinishCompilation(PendingAnimationSequences);
		}
	}
}

#undef LOCTEXT_NAMESPACE //"AnimSequenceCompilingManager"

#endif // WITH_EDITOR