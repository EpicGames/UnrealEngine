// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "IAssetCompilingManager.h"
#include "Containers/Set.h"
#include "Templates/UniquePtr.h"
#include "UObject/WeakObjectPtr.h"

class FAsyncCompilationNotification;
class UAnimSequence;
class USkeleton;
class FQueuedThreadPool;
enum class EQueuedWorkPriority : uint8;

namespace UE::Anim
{
	class FAnimSequenceCompilingManager : public IAssetCompilingManager
	{
	public:
		static FAnimSequenceCompilingManager& Get();

		FAnimSequenceCompilingManager();

		static FName GetStaticAssetTypeName();
	
		virtual void FinishAllCompilation() override;
		virtual void Shutdown() override;
		virtual FName GetAssetTypeName() const override;
		virtual FTextFormat GetAssetNameFormat() const override;
		virtual TArrayView<FName> GetDependentTypeNames() const override;
		virtual int32 GetNumRemainingAssets() const override;
	
		FQueuedThreadPool* GetThreadPool() const;
		EQueuedWorkPriority GetBasePriority(const UAnimSequence* InAnimSequence) const;

		void AddAnimSequences(TArrayView<UAnimSequence* const> InAnimSequences);
		void FinishCompilation(TArrayView<UAnimSequence* const> InAnimSequences);
		void FinishCompilation(TArrayView<USkeleton* const> InSkeletons);

	protected:
		virtual void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;
		void ProcessAnimSequences(bool bLimitExecutionTime, int32 MinBatchSize = 1);

		void PostCompilation(TArrayView<UAnimSequence* const> InAnimSequences);
		void ApplyCompilation(UAnimSequence* InAnimSequence);

		void UpdateCompilationNotification();

		void OnPostReachabilityAnalysis();
	private:
		friend class FAssetCompilingManager;
	
		TSet<TWeakObjectPtr<UAnimSequence>> RegisteredAnimSequences;
		TUniquePtr<FAsyncCompilationNotification> Notification;
		FDelegateHandle PostReachabilityAnalysisHandle;
	};
}

#endif // WITH_EDITOR