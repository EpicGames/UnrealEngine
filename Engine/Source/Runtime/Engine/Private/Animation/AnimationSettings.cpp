// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimationSettings.h"
#include "Animation/AttributeTypes.h"
#include "Animation/CustomAttributes.h"
#include "Animation/MirrorDataTable.h"
#include "StructUtils/UserDefinedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationSettings)

UAnimationSettings::UAnimationSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CompressCommandletVersion(2)	// Bump this up to trigger full recompression. Otherwise only new animations imported will be recompressed.
	, ForceRecompression(false)
	, bForceBelowThreshold(false)
	, bFirstRecompressUsingCurrentOrDefault(true)
	, bRaiseMaxErrorToExisting(false)
	, bEnablePerformanceLog(false)
	, bTickAnimationOnSkeletalMeshInit(true)
	, DefaultAttributeBlendMode(ECustomAttributeBlendType::Blend)
{
	SectionName = TEXT("Animation");

	KeyEndEffectorsMatchNameArray.Add(TEXT("IK"));
	KeyEndEffectorsMatchNameArray.Add(TEXT("eye"));
	KeyEndEffectorsMatchNameArray.Add(TEXT("weapon"));
	KeyEndEffectorsMatchNameArray.Add(TEXT("hand"));
	KeyEndEffectorsMatchNameArray.Add(TEXT("attach"));
	KeyEndEffectorsMatchNameArray.Add(TEXT("camera"));

	MirrorFindReplaceExpressions = {
		FMirrorFindReplaceExpression("r_", "l_", EMirrorFindReplaceMethod::Prefix), FMirrorFindReplaceExpression("l_", "r_", EMirrorFindReplaceMethod::Prefix),
		FMirrorFindReplaceExpression("R_", "L_", EMirrorFindReplaceMethod::Prefix), FMirrorFindReplaceExpression("L_", "R_", EMirrorFindReplaceMethod::Prefix),
		FMirrorFindReplaceExpression("_l", "_r", EMirrorFindReplaceMethod::Suffix), FMirrorFindReplaceExpression("_r", "_l",  EMirrorFindReplaceMethod::Suffix),
		FMirrorFindReplaceExpression("_R", "_L", EMirrorFindReplaceMethod::Suffix), FMirrorFindReplaceExpression("_L", "_R", EMirrorFindReplaceMethod::Suffix),
		FMirrorFindReplaceExpression("right", "left", EMirrorFindReplaceMethod::Prefix), FMirrorFindReplaceExpression("left", "right",  EMirrorFindReplaceMethod::Prefix),
		FMirrorFindReplaceExpression("Right", "Left", EMirrorFindReplaceMethod::Prefix), FMirrorFindReplaceExpression("Left", "Right",  EMirrorFindReplaceMethod::Prefix),
		FMirrorFindReplaceExpression("right", "left", EMirrorFindReplaceMethod::Suffix), FMirrorFindReplaceExpression("left", "right",  EMirrorFindReplaceMethod::Suffix),
		FMirrorFindReplaceExpression("Right", "Left", EMirrorFindReplaceMethod::Suffix), FMirrorFindReplaceExpression("Left", "Right",  EMirrorFindReplaceMethod::Suffix),
		FMirrorFindReplaceExpression("([^}]*)_l_([^}]*)", "$1_r_$2", EMirrorFindReplaceMethod::RegularExpression), FMirrorFindReplaceExpression("([^}]*)_r_([^}]*)", "$1_l_$2", EMirrorFindReplaceMethod::RegularExpression),
		FMirrorFindReplaceExpression("([^}]*)_L_([^}]*)", "$1_R_$2", EMirrorFindReplaceMethod::RegularExpression), FMirrorFindReplaceExpression("([^}]*)_R_([^}]*)", "$1_L_$2", EMirrorFindReplaceMethod::RegularExpression),
		FMirrorFindReplaceExpression("([^}]*)_left_([^}]*)", "$1_right_$2", EMirrorFindReplaceMethod::RegularExpression), FMirrorFindReplaceExpression("([^}]*)_right_([^}]*)", "$1_left_$2", EMirrorFindReplaceMethod::RegularExpression),
		FMirrorFindReplaceExpression("([^}]*)_Left_([^}]*)", "$1_Right_$2", EMirrorFindReplaceMethod::RegularExpression), FMirrorFindReplaceExpression("([^}]*)_Right_([^}]*)", "$1_Left_$2", EMirrorFindReplaceMethod::RegularExpression),
		FMirrorFindReplaceExpression("((?:^[sS]pine|^[rR]oot|^[pP]elvis|^[nN]eck|^[hH]ead|^ik_hand_gun).*)", "$1", EMirrorFindReplaceMethod::RegularExpression)
	};
	
	DefaultFrameRate = FFrameRate(30,1);
	bEnforceSupportedFrameRates = false;
}

TArray<FString> UAnimationSettings::GetBoneCustomAttributeNamesToImport() const
{
	TArray<FString> AttributeNames = {
		BoneTimecodeCustomAttributeNameSettings.HourAttributeName.ToString(),
		BoneTimecodeCustomAttributeNameSettings.MinuteAttributeName.ToString(),
		BoneTimecodeCustomAttributeNameSettings.SecondAttributeName.ToString(),
		BoneTimecodeCustomAttributeNameSettings.FrameAttributeName.ToString(),
		BoneTimecodeCustomAttributeNameSettings.SubframeAttributeName.ToString(),
		BoneTimecodeCustomAttributeNameSettings.RateAttributeName.ToString(),
		BoneTimecodeCustomAttributeNameSettings.TakenameAttributeName.ToString()
	};

	for (const FCustomAttributeSetting& Setting : BoneCustomAttributesNames)
	{
		AttributeNames.AddUnique(Setting.Name);
	}

	return AttributeNames;
}

const FFrameRate& UAnimationSettings::GetDefaultFrameRate() const
{
	checkf(DefaultFrameRate.IsValid(), TEXT("Invalid default frame-rate set in Project Settings"));
	return DefaultFrameRate;
}

#if WITH_EDITOR
void UAnimationSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

 	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimationSettings, UserDefinedStructAttributes))
	{
		TArray<UScriptStruct*> NewListOfStructs;
		for (const TSoftObjectPtr<UUserDefinedStruct>& UserDefinedStruct : UAnimationSettings::Get()->UserDefinedStructAttributes)
		{
			if (UserDefinedStruct.IsValid())
			{
				NewListOfStructs.Add(UserDefinedStruct.Get());
				UE::Anim::AttributeTypes::RegisterNonBlendableType(UserDefinedStruct.Get());
			}
		}

		TArray<TWeakObjectPtr<const UScriptStruct>> RegisteredTypes = UE::Anim::AttributeTypes::GetRegisteredTypes();
		for (const TWeakObjectPtr<const UScriptStruct>& RegisteredStruct : RegisteredTypes)
		{
			if (RegisteredStruct.IsValid() && RegisteredStruct->IsA<UUserDefinedStruct>())
			{
				if (!NewListOfStructs.Contains(RegisteredStruct.Get()))
				{
					UE::Anim::AttributeTypes::UnregisterType(RegisteredStruct.Get());
				}
			}
		}
	}
}


#endif	// WITH_EDITOR

