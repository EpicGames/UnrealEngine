// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AnimationStreaming.cpp: Manager to handle streaming animation data
=============================================================================*/

#include "Animation/AnimationStreaming.h"
#include "Animation/AnimStreamable.h"
#include "Algo/Find.h"
#include "EngineLogs.h"
#include "HAL/PlatformFile.h"
#include "ProfilingDebugging/IoStoreTrace.h"

static int32 SpoofFailedAnimationChunkLoad = 0;
FAutoConsoleVariableRef CVarSpoofFailedAnimationChunkLoad(
	TEXT("a.Streaming.SpoofFailedChunkLoad"),
	SpoofFailedAnimationChunkLoad,
	TEXT("Forces failing to load streamed animation chunks.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);



void FLoadedAnimationChunk::CleanUpIORequest()
{
	if (IORequest)
	{
		IORequest->WaitCompletion();
		delete IORequest;
		IORequest = nullptr;
	}
}

////////////////////////
// FStreamingAnimationData //
////////////////////////

FStreamingAnimationData::FStreamingAnimationData()
	: StreamableAnim(NULL)
	, AnimationStreamingManager(nullptr)
{
	ResetRequestedChunks();
}

FStreamingAnimationData::~FStreamingAnimationData()
{

}

void FStreamingAnimationData::FreeResources()
{
	// Make sure there are no pending requests in flight.
	for (int32 Pass = 0; Pass < 3; Pass++)
	{
		BlockTillAllRequestsFinished();
		if (!UpdateStreamingStatus())
		{
			break;
		}
		check(Pass < 2); // we should be done after two passes. Pass 0 will start anything we need and pass 1 will complete those requests
	}

	for (FLoadedAnimationChunk& LoadedChunk : LoadedChunks)
	{
		FreeLoadedChunk(LoadedChunk);
	}
}

bool FStreamingAnimationData::Initialize(UAnimStreamable* InStreamableAnim, FAnimationStreamingManager* InAnimationStreamingManager)
{
	check(InStreamableAnim && InStreamableAnim->HasRunningPlatformData());

	FStreamableAnimPlatformData& RunningAnimPlatformData = InStreamableAnim->GetRunningPlatformData();

	if (!RunningAnimPlatformData.Chunks.Num())
	{
		UE_LOGF(LogAnimation, Error, "Failed to initialize streaming animation due to lack of anim or serialized stream chunks. '%ls'", *InStreamableAnim->GetFullName());
		return false;
	}

	StreamableAnim = InStreamableAnim;
	AnimationStreamingManager = InAnimationStreamingManager;

	// Always get the first chunk of data so we can play immediately
	check(LoadedChunks.Num() == 0);
	check(LoadedChunkIndices.Num() == 0);

	AddNewLoadedChunk(0, RunningAnimPlatformData.Chunks[0].CompressedAnimSequence);
	LoadedChunkIndices.Add(0);

	return true;
}

bool FStreamingAnimationData::UpdateStreamingStatus()
{
	if (!StreamableAnim)
	{
		return false;
	}

	//Handle Failed Chunks first
	for (int32 LoadedChunkIndex = 0; LoadedChunkIndex < LoadedChunks.Num(); ++LoadedChunkIndex)
	{
		FLoadedAnimationChunk& LoadedChunk = LoadedChunks[LoadedChunkIndex];
		if (LoadFailedChunks.Contains(LoadedChunk.Index))
		{
			// Mark as not loaded
			LoadedChunkIndices.Remove(LoadedChunk.Index);

			// Remove this chunk
			FreeLoadedChunk(LoadedChunk);
			
			FScopeLock LoadedChunksLock(&LoadedChunksCritcalSection);
			LoadedChunks.RemoveAtSwap(LoadedChunkIndex, EAllowShrinking::No);
		}
	}

	LoadFailedChunks.Reset();

	bool bHasPendingRequestInFlight = false;

	TArray<uint32> IndicesToLoad;
	TArray<uint32> IndicesToFree;

	if (HasPendingRequests(IndicesToLoad, IndicesToFree))
	{
		// could maybe iterate over the things we know are done, but I couldn't tell if that was IndicesToLoad or not.
		for (FLoadedAnimationChunk& LoadedChunk : LoadedChunks)
		{
			if (LoadedChunk.IORequest)
			{
				const bool bRequestFinished = LoadedChunk.IORequest->PollCompletion();
				bHasPendingRequestInFlight |= !bRequestFinished;
				if (bRequestFinished)
				{
					LoadedChunk.CleanUpIORequest();
				}
			}
		}

		LoadedChunkIndices = RequestedChunks;

		BeginPendingRequests(IndicesToLoad, IndicesToFree);
	}

	ResetRequestedChunks();

	return bHasPendingRequestInFlight;
}

bool FStreamingAnimationData::HasPendingRequests(TArray<uint32>& IndicesToLoad, TArray<uint32>& IndicesToFree) const
{
	IndicesToLoad.Reset();
	IndicesToFree.Reset();

	// Find indices that aren't loaded
	for (uint32 NeededIndex : RequestedChunks)
	{
		if (!LoadedChunkIndices.Contains(NeededIndex))
		{
			IndicesToLoad.AddUnique(NeededIndex);
		}
	}

	// Find indices that aren't needed anymore
	for (uint32 CurrentIndex : LoadedChunkIndices)
	{
		if (!RequestedChunks.Contains(CurrentIndex))
		{
			IndicesToFree.AddUnique(CurrentIndex);
		}
	}

	return IndicesToLoad.Num() > 0 || IndicesToFree.Num() > 0;
}

void FStreamingAnimationData::BeginPendingRequests(const TArray<uint32>& IndicesToLoad, const TArray<uint32>& IndicesToFree)
{
	TArray<uint32> FreeChunkIndices;

	// Mark Chunks for removal in case they can be reused
	{
		for (uint32 Index : IndicesToFree)
		{
			for (int32 ChunkIndex = 0; ChunkIndex < LoadedChunks.Num(); ++ChunkIndex)
			{
				check(Index != 0);
				if (LoadedChunks[ChunkIndex].Index == Index)
				{
					FreeLoadedChunk(LoadedChunks[ChunkIndex]);

					FScopeLock LoadedChunksLock(&LoadedChunksCritcalSection);
					LoadedChunks.RemoveAtSwap(ChunkIndex,EAllowShrinking::No);
					break;
				}
			}
		}
	}

	// Set off all IO Requests

	const EAsyncIOPriorityAndFlags AsyncIOPriority = AIOP_CriticalPath; //Set to Crit temporarily as emergency speculative fix for streaming issue

	for (uint32 ChunkIndex : IndicesToLoad)
	{
		const FAnimStreamableChunk& Chunk = StreamableAnim->GetRunningPlatformData().Chunks[ChunkIndex];

		FCompressedAnimSequence* ExistingCompressedData = Chunk.CompressedAnimSequence;

		FLoadedAnimationChunk& ChunkStorage = AddNewLoadedChunk(ChunkIndex, ExistingCompressedData);

		if(!ExistingCompressedData)
		{
			UE_CLOGF(ChunkStorage.CompressedAnimData != nullptr, LogAnimation, Fatal, "Existing compressed data for streaming animation chunk.");
			UE_CLOGF(ChunkStorage.IORequest, LogAnimation, Fatal, "Streaming animation chunk already has IORequest.");
			
			int64 ChunkSize = Chunk.BulkData.GetBulkDataSize();
			ChunkStorage.RequestStart = FPlatformTime::Seconds();
			UE_LOGF(LogAnimation, Warning, "Anim Streaming Request Started %ls Chunk:%i At:%.3f\n", *StreamableAnim->GetName(), ChunkIndex, ChunkStorage.RequestStart);
			FBulkDataIORequestCallBack AsyncFileCallBack =
				[this, ChunkIndex, ChunkSize](bool bWasCancelled, IBulkDataIORequest* Req)
			{
				AnimationStreamingManager->OnAsyncFileCallback(this, ChunkIndex, ChunkSize, Req, bWasCancelled);
			};

			UE_LOG(LogAnimation, Warning, TEXT("Loading Stream Anim %s Chunk:%i Length: %.3f Offset:%" INT64_FMT " Size:%" INT64_FMT " File:%s\n"),
				*StreamableAnim->GetName(), ChunkIndex, Chunk.SequenceLength, Chunk.BulkData.GetBulkDataOffsetInFile(), Chunk.BulkData.GetBulkDataSize(), *Chunk.BulkData.GetDebugName());
			TRACE_IOSTORE_METADATA_SCOPE_TAG(*StreamableAnim->GetName());
			ChunkStorage.IORequest = Chunk.BulkData.CreateStreamingRequest(AsyncIOPriority, &AsyncFileCallBack, nullptr);
			if (!ChunkStorage.IORequest)
			{
				UE_LOGF(LogAnimation, Error, "Animation streaming read request failed.");
			}
		}
	}
}

bool FStreamingAnimationData::BlockTillAllRequestsFinished(float TimeLimit)
{
	QUICK_SCOPE_CYCLE_COUNTER(FStreamingAnimData_BlockTillAllRequestsFinished);
	if (TimeLimit == 0.0f)
	{
		for (FLoadedAnimationChunk& LoadedChunk : LoadedChunks)
		{
			LoadedChunk.CleanUpIORequest();
		}
	}
	else
	{
		const double EndTime = FPlatformTime::Seconds() + TimeLimit;
		for (FLoadedAnimationChunk& LoadedChunk : LoadedChunks)
		{
			if (LoadedChunk.IORequest)
			{
				const float ThisTimeLimit = static_cast<float>(EndTime - FPlatformTime::Seconds());
				if (ThisTimeLimit < .001f || // one ms is the granularity of the platform event system
					!LoadedChunk.IORequest->WaitCompletion(ThisTimeLimit))
				{
					return false;
				}

				LoadedChunk.CleanUpIORequest();
			}
		}
	}
	return true;
}

FLoadedAnimationChunk& FStreamingAnimationData::AddNewLoadedChunk(uint32 ChunkIndex, FCompressedAnimSequence* ExistingData)
{
	FScopeLock LoadedChunksLock(&LoadedChunksCritcalSection);
	int32 NewIndex = LoadedChunks.Num();
	LoadedChunks.AddDefaulted();

	LoadedChunks[NewIndex].CompressedAnimData = ExistingData;
	LoadedChunks[NewIndex].bOwnsCompressedData = false;
	LoadedChunks[NewIndex].Index = ChunkIndex;
	return LoadedChunks[NewIndex];
}

void FStreamingAnimationData::FreeLoadedChunk(FLoadedAnimationChunk& LoadedChunk)
{
	if (LoadedChunk.IORequest)
	{
		LoadedChunk.IORequest->Cancel();
		LoadedChunk.IORequest->WaitCompletion();
		delete LoadedChunk.IORequest;
		LoadedChunk.IORequest = nullptr;
	}

	if (LoadedChunk.bOwnsCompressedData)
	{
		delete LoadedChunk.CompressedAnimData.Load();
	}

	LoadedChunk.CompressedAnimData = NULL;
	LoadedChunk.bOwnsCompressedData = false;
	LoadedChunk.Index = 0;
}

void FStreamingAnimationData::ResetRequestedChunks()
{
	RequestedChunks.Reset();
	RequestedChunks.Add(0); //Always want 1
}

SIZE_T FStreamingAnimationData::GetMemorySize() const
{
	SIZE_T CurrentSize = 0;
	for (const FLoadedAnimationChunk& Chunk : LoadedChunks)
	{
		if (Chunk.CompressedAnimData && Chunk.bOwnsCompressedData)
		{
			CurrentSize += Chunk.CompressedAnimData.Load()->GetMemorySize();
		}
	}
	return CurrentSize;
}

////////////////////////////
// FAnimationStreamingManager //
////////////////////////////

FAnimationStreamingManager::FAnimationStreamingManager()
{
}

FAnimationStreamingManager::~FAnimationStreamingManager()
{
}

void FAnimationStreamingManager::OnAsyncFileCallback(FStreamingAnimationData* StreamingAnimData, int32 ChunkIndex, int64 ReadSize, IBulkDataIORequest* ReadRequest, bool bWasCancelled)
{
	// Check to see if we successfully managed to load anything
	uint8* Mem = ReadRequest->GetReadResults();

	FScopeLock Lock(&StreamingAnimData->LoadedChunksCritcalSection);

	const int32 LoadedChunkIndex = StreamingAnimData->LoadedChunks.IndexOfByPredicate([ChunkIndex](const FLoadedAnimationChunk& Chunk) { return Chunk.Index == ChunkIndex; });
	FLoadedAnimationChunk& ChunkStorage = StreamingAnimData->LoadedChunks[LoadedChunkIndex];

	const double CurrentTime = FPlatformTime::Seconds();
	const double RequestDuration = CurrentTime - ChunkStorage.RequestStart;

	if (Mem)
	{

		checkf(ChunkStorage.CompressedAnimData == nullptr, TEXT("Chunk storage already has data. (0x%p) ChunkIndex:%i LoadChunkIndex:%i"), ChunkStorage.CompressedAnimData.Load(), ChunkIndex, LoadedChunkIndex);

		FCompressedAnimSequence* NewCompressedData = new FCompressedAnimSequence();

		FMemoryView MemView(Mem, ReadSize);
		FMemoryReaderView Reader(MemView);

		UAnimStreamable* Anim = StreamingAnimData->StreamableAnim;
		NewCompressedData->SerializeCompressedData(Reader, false, Anim, Anim->GetSkeleton(), Anim->BoneCompressionSettings, Anim->CurveCompressionSettings);

		const float SequenceLength = StreamingAnimData->StreamableAnim->GetRunningPlatformData().Chunks[ChunkIndex].SequenceLength;
		if (SequenceLength < RequestDuration)
		{
			UE_LOGF(LogAnimation, Warning, "Streaming anim loading slower than needed ChunkIndex %i - Length: %.3f Load Duration:%.3f Anim:%ls\n", ChunkIndex, SequenceLength, RequestDuration, *StreamingAnimData->StreamableAnim->GetName());
		}

		ChunkStorage.CompressedAnimData = NewCompressedData;
		ChunkStorage.bOwnsCompressedData = true;
		ChunkStorage.RequestStart = -2.0; //Signify we have finished loading

		UE_LOGF(LogAnimation, Log, "Request Finished %.2f\n Anim Chunk Streamed %.4f\n", CurrentTime, RequestDuration);

		FMemory::Free(Mem);
	}
	else
	{
		const TCHAR* WasCancelledText = bWasCancelled ? TEXT("Yes") : TEXT("No");
		UE_LOGF(LogAnimation, Warning, "Streaming anim failed to load chunk: %i Load Duration:%.3f, Anim:%ls WasCancelled: %ls\n", ChunkIndex, RequestDuration, *StreamingAnimData->StreamableAnim->GetName(), WasCancelledText);
		
		StreamingAnimData->LoadFailedChunks.AddUnique(ChunkIndex);
	}
}

SIZE_T FAnimationStreamingManager::GetMemorySizeForAnim(const UAnimStreamable* Anim)
{
	SIZE_T AnimSize = 0;

	FScopeLock Lock(&CriticalSection);

	FStreamingAnimationData* AnimData = StreamingAnimations.FindRef(Anim);
	if (AnimData)
	{
		return AnimData->GetMemorySize();
	}

	return AnimSize;
}

void FAnimationStreamingManager::UpdateResourceStreaming(float DeltaTime, bool bProcessEverything /*= false*/)
{
	LLM_SCOPE(ELLMTag::Animation);

	FScopeLock Lock(&CriticalSection);

	for (TPair<UAnimStreamable*, FStreamingAnimationData*>& AnimData : StreamingAnimations)
	{
		AnimData.Value->UpdateStreamingStatus();
	}
}

int32 FAnimationStreamingManager::BlockTillAllRequestsFinished(float TimeLimit, bool)
{
	{
		FScopeLock Lock(&CriticalSection);

		QUICK_SCOPE_CYCLE_COUNTER(FAnimStreamingManager_BlockTillAllRequestsFinished);
		int32 Result = 0;

		if (TimeLimit == 0.0f)
		{
			for (TPair<UAnimStreamable*, FStreamingAnimationData*>& AnimPair : StreamingAnimations)
			{
				AnimPair.Value->BlockTillAllRequestsFinished();
			}
		}
		else
		{
			const double EndTime = FPlatformTime::Seconds() + TimeLimit;
			for (TPair<UAnimStreamable*, FStreamingAnimationData*>& AnimPair : StreamingAnimations)
			{
				const float ThisTimeLimit = static_cast<float>(EndTime - FPlatformTime::Seconds());
				if (ThisTimeLimit < .001f || // one ms is the granularity of the platform event system
					!AnimPair.Value->BlockTillAllRequestsFinished(ThisTimeLimit))
				{
					Result = 1; // we don't report the actual number, just 1 for any number of outstanding requests
					break;
				}
			}
		}

		return Result;
	}
}

void FAnimationStreamingManager::CancelForcedResources()
{
}

void FAnimationStreamingManager::NotifyLevelChange()
{
}

void FAnimationStreamingManager::SetDisregardWorldResourcesForFrames(int32 NumFrames)
{
}

void FAnimationStreamingManager::AddLevel(class ULevel* Level)
{
}

void FAnimationStreamingManager::RemoveLevel(class ULevel* Level)
{
}

void FAnimationStreamingManager::NotifyLevelOffset(class ULevel* Level, const FVector& Offset)
{
}

void FAnimationStreamingManager::AddStreamingAnim(UAnimStreamable* Anim)
{
	FScopeLock Lock(&CriticalSection);
	if (StreamingAnimations.FindRef(Anim) == nullptr)
	{
		FStreamingAnimationData* NewStreamingAnim = new FStreamingAnimationData;
		if (NewStreamingAnim->Initialize(Anim, this))
		{
			StreamingAnimations.Add(Anim, NewStreamingAnim);
		}
		else
		{
			// Failed to initialize, don't add to list of streaming sound waves
			delete NewStreamingAnim;
		}
	}
}

bool FAnimationStreamingManager::RemoveStreamingAnim(UAnimStreamable* Anim)
{
	FScopeLock Lock(&CriticalSection);
	FStreamingAnimationData* AnimData = StreamingAnimations.FindRef(Anim);
	if (AnimData)
	{
		StreamingAnimations.Remove(Anim);

		// Free the resources of the streaming wave data. This blocks pending IO requests
		AnimData->FreeResources();
		delete AnimData;
		return true;
	}
	return false;
}

const FCompressedAnimSequence* FAnimationStreamingManager::GetLoadedChunk(const UAnimStreamable* Anim, uint32 ChunkIndex, bool bTrackAsRequested) const
{
	// Check for the spoof of failing to load a stream chunk
	if (SpoofFailedAnimationChunkLoad > 0)
	{
		return nullptr;
	}

	// If we fail at getting the critical section here, early out.
	FScopeLock MapLock(&CriticalSection);

	FStreamingAnimationData* AnimData = StreamingAnimations.FindRef(Anim);
	if (AnimData)
	{
		if (bTrackAsRequested)
		{
			AnimData->RequestedChunks.AddUnique(ChunkIndex);
			AnimData->RequestedChunks.AddUnique((ChunkIndex + 1) % Anim->GetRunningPlatformData().Chunks.Num());
		}

		if (AnimData->LoadedChunkIndices.Contains(ChunkIndex))
		{
			if (const FLoadedAnimationChunk* Chunk = Algo::FindBy(AnimData->LoadedChunks, ChunkIndex, &FLoadedAnimationChunk::Index))
			{
				if (Chunk->CompressedAnimData == nullptr)
				{
					const double RequestTimer = Chunk->RequestStart < 0.f ? Chunk->RequestStart : FPlatformTime::Seconds() - Chunk->RequestStart;
					UE_LOGF(LogAnimation, Warning, "No Animation Data for loaded chunk: %i, Anim: %ls Request timer : %.3f", ChunkIndex, *Anim->GetFullName(), RequestTimer);
				}
				return Chunk->CompressedAnimData;
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Unable to find requested Chunk: %i, Anim: %ls - Is in LoadedChunkIndices however", ChunkIndex, *Anim->GetFullName());
			}
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Requested Previously Unknown Chunk: %i, Anim: %ls",  ChunkIndex, *Anim->GetFullName());
		}
	}

	return nullptr;
}
