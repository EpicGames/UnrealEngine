// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationUtils.cpp: Skeletal mesh animation utilities.
=============================================================================*/ 

#include "AnimationUtils.h"
#include "Animation/AnimSequence.h"
#include "Misc/ConfigCacheIni.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "UObject/Package.h"
#include "Animation/Skeleton.h"
#include "AnimationRuntime.h"
#include "Animation/AnimSet.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AnimBoneCompressionCodec.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimCurveCompressionSettings.h"
#include "Animation/VariableFrameStrippingSettings.h"
#include "AnimationCompression.h"
#include "Engine/SkeletalMeshSocket.h"
#include "UObject/LinkerLoad.h"

/** Array to keep track of SkeletalMeshes we have built metadata for, and log out the results just once. */
//static TArray<USkeleton*> UniqueSkeletonsMetadataArray;

void FAnimationUtils::BuildSkeletonMetaData(USkeleton* Skeleton, TArray<FBoneData>& OutBoneData)
{
	// Disable logging by default. Except if we deal with a new Skeleton. Then we log out its details. (just once).
	bool bEnableLogging = false;
// Uncomment to enable.
// 	if( UniqueSkeletonsMetadataArray.FindItemIndex(Skeleton) == INDEX_NONE )
// 	{
// 		bEnableLogging = true;
// 		UniqueSkeletonsMetadataArray.AddItem(Skeleton);
// 	}

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	const TArray<FTransform> & SkeletonRefPose = Skeleton->GetRefLocalPoses();
	const int32 NumBones = RefSkeleton.GetNum();

	// Assemble bone data.
	OutBoneData.Empty();
	OutBoneData.AddZeroed( NumBones );

	const TArray<FString>& KeyEndEffectorsMatchNameArray = UAnimationSettings::Get()->KeyEndEffectorsMatchNameArray;

	for (int32 BoneIndex = 0 ; BoneIndex<NumBones; ++BoneIndex )
	{
		FBoneData& BoneData = OutBoneData[BoneIndex];

		// Copy over data from the skeleton.
		const FTransform& SrcTransform = SkeletonRefPose[BoneIndex];

		ensure(!SrcTransform.ContainsNaN());
		ensure(SrcTransform.IsRotationNormalized());

		BoneData.Orientation = SrcTransform.GetRotation();
		BoneData.Position = (FVector3f)SrcTransform.GetTranslation();
		BoneData.Scale = (FVector3f)SrcTransform.GetScale3D();
		BoneData.Name = RefSkeleton.GetBoneName(BoneIndex);

		if ( BoneIndex > 0 )
		{
			// Compute ancestry.
			int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			BoneData.BonesToRoot.Add( ParentIndex );
			while ( ParentIndex > 0 )
			{
				ParentIndex = RefSkeleton.GetParentIndex(ParentIndex);
				BoneData.BonesToRoot.Add( ParentIndex );
			}
		}

		// See if a Socket is attached to that bone
		BoneData.bHasSocket = false;
		// @todo anim: socket isn't moved to Skeleton yet, but this code needs better testing
		for(int32 SocketIndex=0; SocketIndex<Skeleton->Sockets.Num(); SocketIndex++)
		{
			USkeletalMeshSocket* Socket = Skeleton->Sockets[SocketIndex];
			if( Socket && Socket->BoneName == RefSkeleton.GetBoneName(BoneIndex) )
			{
				BoneData.bHasSocket = true;
				break;
			}
		}
	}

	// Enumerate children (bones that refer to this bone as parent).
	for(int32 BoneIndex = 1; BoneIndex < OutBoneData.Num(); ++BoneIndex)
	{
		const int32 ParentIndex = OutBoneData[BoneIndex].GetParent();
		if (OutBoneData.IsValidIndex(ParentIndex))
		{
			OutBoneData[ParentIndex].Children.Add(BoneIndex);
		}
	}
	
	// Enumerate end effectors.  For each end effector, propagate its index up to all ancestors.
	if( bEnableLogging )
	{
		UE_LOGF(LogAnimationCompression, Log, "Enumerate End Effectors for %ls", *Skeleton->GetFName().ToString());
	}
	for ( int32 BoneIndex = 0 ; BoneIndex < OutBoneData.Num() ; ++BoneIndex )
	{
		FBoneData& BoneData = OutBoneData[BoneIndex];
		if ( BoneData.IsEndEffector() )
		{
			// End effectors have themselves as an ancestor.
			BoneData.EndEffectors.Add( BoneIndex );
			// Add the end effector to the list of end effectors of all ancestors.
			for ( int32 i = 0 ; i < BoneData.BonesToRoot.Num() ; ++i )
			{
				const int32 AncestorIndex = BoneData.BonesToRoot[i];
				OutBoneData[AncestorIndex].EndEffectors.Add( BoneIndex );
			}

			for(int32 MatchIndex=0; MatchIndex<KeyEndEffectorsMatchNameArray.Num(); MatchIndex++)
			{
				// See if this bone has been defined as a 'key' end effector
				FString BoneString(BoneData.Name.ToString());
				if( BoneString.Contains(KeyEndEffectorsMatchNameArray[MatchIndex]) )
				{
					BoneData.bKeyEndEffector = true;
					break;
				}
			}
			if( bEnableLogging )
			{
				UE_LOGF(LogAnimationCompression, Log, "\t %ls bKeyEndEffector: %d", *BoneData.Name.ToString(), BoneData.bKeyEndEffector);
			}
		}
	}
#if 0
	UE_LOGF(LogAnimationCompression, Log,  "====END EFFECTORS:" );
	int32 NumEndEffectors = 0;
	for ( int32 BoneIndex = 0 ; BoneIndex < OutBoneData.Num() ; ++BoneIndex )
	{
		const FBoneData& BoneData = OutBoneData(BoneIndex);
		if ( BoneData.IsEndEffector() )
		{
			FString Message( FString::Printf(TEXT("%s(%i): "), *BoneData.Name, BoneData.GetDepth()) );
			for ( int32 i = 0 ; i < BoneData.BonesToRoot.Num() ; ++i )
			{
				const int32 AncestorIndex = BoneData.BonesToRoot(i);
				Message += FString::Printf( TEXT("%s "), *OutBoneData(AncestorIndex).Name );
			}
			UE_LOG(LogAnimation, Log,  *Message );
			++NumEndEffectors;
		}
	}
	UE_LOGF(LogAnimationCompression, Log,  "====NUM EFFECTORS %i(%i)", NumEndEffectors, OutBoneData(0).Children.Num() );
	UE_LOGF(LogAnimationCompression, Log,  "====NON END EFFECTORS:" );
	for ( int32 BoneIndex = 0 ; BoneIndex < OutBoneData.Num() ; ++BoneIndex )
	{
		const FBoneData& BoneData = OutBoneData(BoneIndex);
		if ( !BoneData.IsEndEffector() )
		{
			FString Message( FString::Printf(TEXT("%s(%i): "), *BoneData.Name, BoneData.GetDepth()) );
			Message += TEXT("Children: ");
			for ( int32 i = 0 ; i < BoneData.Children.Num() ; ++i )
			{
				const int32 ChildIndex = BoneData.Children(i);
				Message += FString::Printf( TEXT("%s "), *OutBoneData(ChildIndex).Name );
			}
			Message += TEXT("  EndEffectors: ");
			for ( int32 i = 0 ; i < BoneData.EndEffectors.Num() ; ++i )
			{
				const int32 EndEffectorIndex = BoneData.EndEffectors(i);
				Message += FString::Printf( TEXT("%s "), *OutBoneData(EndEffectorIndex).Name );
				check( OutBoneData(EndEffectorIndex).IsEndEffector() );
			}
			UE_LOG(LogAnimationCompression, Log,  *Message );
		}
	}
	UE_LOGF(LogAnimationCompression, Log,  "===================" );
#endif
}

/**
* Builds the local-to-component matrix for the specified bone.
*/
void FAnimationUtils::BuildComponentSpaceTransform(FTransform& OutTransform,
											   int32 BoneIndex,
											   const TArray<FTransform>& BoneSpaceTransforms,
											   const TArray<FBoneData>& BoneData)
{
	// Put root-to-component in OutTransform.
	OutTransform = BoneSpaceTransforms[0];

	if (BoneIndex > 0)
	{
		const FBoneData& Bone = BoneData[BoneIndex];

		checkSlow((Bone.BonesToRoot.Num() - 1) == 0);

		// Compose BoneData.BonesToRoot down.
		for (int32 i = Bone.BonesToRoot.Num()-2; i >=0; --i)
		{
			const int32 AncestorIndex = Bone.BonesToRoot[i];
			ensure(AncestorIndex != INDEX_NONE);
			OutTransform = BoneSpaceTransforms[AncestorIndex] * OutTransform;
			OutTransform.NormalizeRotation();
		}

		// Finally, include the bone's local-to-parent.
		OutTransform = BoneSpaceTransforms[BoneIndex] * OutTransform;
		OutTransform.NormalizeRotation();
	}
}

int32 FAnimationUtils::GetAnimTrackIndexForSkeletonBone(const int32 InSkeletonBoneIndex, const TArray<FTrackToSkeletonMap>& TrackToSkelMap)
{
	return TrackToSkelMap.IndexOfByPredicate([&](const FTrackToSkeletonMap& TrackToSkel)
	{ 
		return TrackToSkel.BoneTreeIndex == InSkeletonBoneIndex;
	});
}

#if WITH_EDITOR
/**
 * Utility function to measure the accuracy of a compressed animation. Each end-effector is checked for 
 * world-space movement as a result of compression.
 *
 * @param	AnimSet		The animset to calculate compression error for.
 * @param	BoneData	BoneData array describing the hierarchy of the animated skeleton
 * @param	ErrorStats	Output structure containing the final compression error values
 * @return				None.
 */
void FAnimationUtils::ComputeCompressionError(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& CompressedData, AnimationErrorStats& ErrorStats)
{
	ErrorStats.AverageError = 0.0f;
	ErrorStats.MaxError = 0.0f;
	ErrorStats.MaxErrorBone = 0;
	ErrorStats.MaxErrorTime = 0.0f;
	int32 MaxErrorTrack = -1;

	if (CompressedData.AnimData != nullptr && CompressedData.AnimData->CompressedNumberOfKeys > 0)
	{
		const bool bCanUseCompressedData = CompressedData.AnimData->IsValid();
		if (!bCanUseCompressedData)
		{
			// If we can't use CompressedData, there's not much point in being here.
			return;
		}

		const int32 NumBones = CompressibleAnimData.BoneData.Num();
		
		float ErrorCount = 0.0f;
		float ErrorTotal = 0.0f;

		const TArray<FTransform>& RefPose = CompressibleAnimData.RefLocalPoses;

		TArray<FTransform> RawTransforms;
		TArray<FTransform> NewTransforms;
		RawTransforms.AddZeroed(NumBones);
		NewTransforms.AddZeroed(NumBones);

		// Cache these to speed up animations with a lot of frames.
		// We do this only once, instead of every frame.
		struct FCachedBoneIndexData
		{
			int32 TrackIndex;
			int32 ParentIndex;
		};
		TArray<FCachedBoneIndexData> CachedBoneIndexData;
		CachedBoneIndexData.AddZeroed(NumBones);
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			FCachedBoneIndexData& BoneIndexData = CachedBoneIndexData[BoneIndex];
			BoneIndexData.TrackIndex = GetAnimTrackIndexForSkeletonBone(BoneIndex, CompressibleAnimData.TrackToSkeletonMapTable);
			BoneIndexData.ParentIndex = CompressibleAnimData.RefSkeleton.GetParentIndex(BoneIndex);
		}

		// Check the precondition that parents occur before children in the RequiredBones array.
		for (int32 BoneIndex = 1; BoneIndex < NumBones; ++BoneIndex)
		{
			const FCachedBoneIndexData& BoneIndexData = CachedBoneIndexData[BoneIndex];
			check(BoneIndexData.ParentIndex != INDEX_NONE);
			check(BoneIndexData.ParentIndex < BoneIndex);
		}

		const FTransform EndEffectorDummyBoneSocket(FQuat::Identity, FVector(END_EFFECTOR_DUMMY_BONE_LENGTH_SOCKET));
		const FTransform EndEffectorDummyBone(FQuat::Identity, FVector(END_EFFECTOR_DUMMY_BONE_LENGTH));

		FAnimSequenceDecompressionContext DecompContext(CompressibleAnimData.SampledFrameRate, CompressibleAnimData.GetNumberOfFrames(), CompressibleAnimData.Interpolation, CompressibleAnimData.AnimFName, *CompressedData.AnimData, RefPose,
			CompressibleAnimData.TrackToSkeletonMapTable, nullptr, CompressibleAnimData.bIsValidAdditive, CompressibleAnimData.AdditiveType);

		const TArray<FBoneData>& BoneData = CompressibleAnimData.BoneData;

		for (int32 KeyIndex = 0; KeyIndex< CompressedData.AnimData->CompressedNumberOfKeys; KeyIndex++)
		{
			const double Time = CompressibleAnimData.SampledFrameRate.AsSeconds(KeyIndex);
			DecompContext.Seek(Time);

			// get the raw and compressed atom for each bone
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				const FCachedBoneIndexData& BoneIndexData = CachedBoneIndexData[BoneIndex];
				if (BoneIndexData.TrackIndex == INDEX_NONE)
				{
					// No track for the bone was found, use default transform
					const FTransform& RefPoseTransform = RefPose[BoneIndex];
					RawTransforms[BoneIndex] = RefPoseTransform;
					NewTransforms[BoneIndex] = RefPoseTransform;
				}
				else
				{
					// If we have transforms, but they're additive, apply them to the ref pose.
					// This is because additive animations are mostly rotation.
					// And for the error metric we measure distance between end effectors.
					// So that means additive animations by default will all be balled up at the origin and not show any error.
					if (CompressibleAnimData.bIsValidAdditive)
					{
						const FTransform& RefPoseTransform = RefPose[BoneIndex];
						RawTransforms[BoneIndex] = RefPoseTransform;
						NewTransforms[BoneIndex] = RefPoseTransform;

						FTransform AdditiveRawTransform;
						FTransform AdditiveNewTransform;
						FAnimationUtils::ExtractTransformFromTrack(CompressibleAnimData.RawAnimationData[BoneIndexData.TrackIndex], Time, CompressibleAnimData.NumberOfKeys, CompressibleAnimData.SequenceLength, CompressibleAnimData.Interpolation, AdditiveRawTransform);
						CompressedData.Codec->DecompressBone(DecompContext, BoneIndexData.TrackIndex, AdditiveNewTransform);

						const ScalarRegister VBlendWeight(1.f);
						RawTransforms[BoneIndex].AccumulateWithAdditiveScale(AdditiveRawTransform, VBlendWeight);
						NewTransforms[BoneIndex].AccumulateWithAdditiveScale(AdditiveNewTransform, VBlendWeight);
					}
					else
					{
						FAnimationUtils::ExtractTransformFromTrack(CompressibleAnimData.RawAnimationData[BoneIndexData.TrackIndex], Time, CompressibleAnimData.NumberOfKeys, CompressibleAnimData.SequenceLength,  CompressibleAnimData.Interpolation, RawTransforms[BoneIndex]);
						CompressedData.Codec->DecompressBone(DecompContext, BoneIndexData.TrackIndex, NewTransforms[BoneIndex]);
					}
				}

				ensure(!RawTransforms[BoneIndex].ContainsNaN());
				ensure(!NewTransforms[BoneIndex].ContainsNaN());

				// For all bones below the root, final component-space transform is relative transform * component-space transform of parent.
				if (BoneIndex > 0)
				{
					RawTransforms[BoneIndex] *= RawTransforms[BoneIndexData.ParentIndex];
					NewTransforms[BoneIndex] *= NewTransforms[BoneIndexData.ParentIndex];
				}
				
				// If this is an EndEffector, add a dummy bone to measure the effect of compressing the rotation.
				if (BoneData[BoneIndex].IsEndEffector())
				{
					// Sockets and Key EndEffectors have a longer dummy bone to maintain higher precision.
					if (BoneData[BoneIndex].bHasSocket || BoneData[BoneIndex].bKeyEndEffector)
					{
						RawTransforms[BoneIndex] = EndEffectorDummyBoneSocket * RawTransforms[BoneIndex];
						NewTransforms[BoneIndex] = EndEffectorDummyBoneSocket * NewTransforms[BoneIndex];
					}
					else
					{
						RawTransforms[BoneIndex] = EndEffectorDummyBone * RawTransforms[BoneIndex];
						NewTransforms[BoneIndex] = EndEffectorDummyBone * NewTransforms[BoneIndex];
					}
				}

				// Normalize rotations
				RawTransforms[BoneIndex].NormalizeRotation();
				NewTransforms[BoneIndex].NormalizeRotation();

				if (BoneData[BoneIndex].IsEndEffector())
				{
					float Error = (RawTransforms[BoneIndex].GetLocation() - NewTransforms[BoneIndex].GetLocation()).Size();

					ErrorTotal += Error;
					ErrorCount += 1.0f;

					if (Error > ErrorStats.MaxError)
					{
						ErrorStats.MaxError	= Error;
						ErrorStats.MaxErrorBone = BoneIndex;
						MaxErrorTrack = BoneIndexData.TrackIndex;
						ErrorStats.MaxErrorTime = Time;
					}
				}
			}
		}

		if (ErrorCount > 0.0f)
		{
			ErrorStats.AverageError = ErrorTotal / ErrorCount;
		}
	}
}
#endif

/**
 * Determines the current setting for recompressing all animations upon load. The default value 
 * is False, but may be overridden by an optional field in the base engine INI file. 
 *
 * @return				true if the engine settings request that all animations be recompiled
 */
bool FAnimationUtils::GetForcedRecompressionSetting()
{
	// Allow the Engine INI file to provide a new override
	bool ForcedRecompressionSetting = false;
	GConfig->GetBool( TEXT("AnimationCompression"), TEXT("ForceRecompression"), (bool&)ForcedRecompressionSetting, GEngineIni );

	return ForcedRecompressionSetting;
}

static void GetBindPoseAtom(FTransform &OutBoneAtom, int32 BoneIndex, USkeleton *Skeleton)
{
	OutBoneAtom = Skeleton->GetRefLocalPoses()[BoneIndex];
// #if DEBUG_ADDITIVE_CREATION
// 	UE_LOGF(LogAnimation, Log, "GetBindPoseAtom BoneIndex: %d, OutBoneAtom: %ls", BoneIndex, *OutBoneAtom.ToString());
// #endif
}

/** Get default Outer for AnimSequences contained in this AnimSet.
	*  The intent is to use that when Constructing new AnimSequences to put into that set.
	*  The Outer will be Package.<AnimSetName>_Group.
	*  @param bCreateIfNotFound if true, Group will be created. This is only in the editor.
	*/
UObject* FAnimationUtils::GetDefaultAnimSequenceOuter(UAnimSet* InAnimSet, bool bCreateIfNotFound)
{
	check( InAnimSet );

#if WITH_EDITORONLY_DATA
	for(int32 i=0; i<InAnimSet->Sequences.Num(); i++)
	{
		UAnimSequence* TestAnimSeq = InAnimSet->Sequences[i];
		// Make sure outer is not current AnimSet, but they should be in the same package.
		if( TestAnimSeq && TestAnimSeq->GetOuter() != InAnimSet && TestAnimSeq->GetOutermost() == InAnimSet->GetOutermost() )
		{
			return TestAnimSeq->GetOuter();
		}
	}
#endif	//#if WITH_EDITORONLY_DATA

	// Otherwise go ahead and create a new one if we should.
	if( bCreateIfNotFound )
	{
		// We can only create Group if we are within the editor.
		check(GIsEditor);

		UPackage* AnimSetPackage = InAnimSet->GetOutermost();
		// Make sure package is fully loaded.
		AnimSetPackage->FullyLoad();

		// Try to create a new package with Group named <AnimSetName>_Group.
		FString NewPackageString = FString::Printf(TEXT("%s.%s_Group"), *AnimSetPackage->GetFName().ToString(), *InAnimSet->GetFName().ToString());
		UPackage* NewPackage = CreatePackage( *NewPackageString );

		// New Outer to use
		return NewPackage;
	}

	return nullptr;
}


/**
 * Converts an animation compression type into a human readable string
 *
 * @param	InFormat	The compression format to convert into a string
 * @return				The format as a string
 */
FString FAnimationUtils::GetAnimationCompressionFormatString(AnimationCompressionFormat InFormat)
{
	switch(InFormat)
	{
	case ACF_None:
		return FString("ACF_None");
	case ACF_Float96NoW:
		return FString("ACF_Float96NoW");
	case ACF_Fixed48NoW:
		return FString("ACF_Fixed48NoW");
	case ACF_IntervalFixed32NoW:
		return FString("ACF_IntervalFixed32NoW");
	case ACF_Fixed32NoW:
		return FString("ACF_Fixed32NoW");
	case ACF_Float32NoW:
		return FString("ACF_Float32NoW");
	case ACF_Identity:
		return FString("ACF_Identity");
	default:
		UE_LOGF(LogAnimationCompression, Warning, "AnimationCompressionFormat was not found:  %i", static_cast<int32>(InFormat) );
	}

	return FString("Unknown");
}

/**
 * Converts an animation codec format into a human readable string
 *
 * @param	InFormat	The format to convert into a string
 * @return				The format as a string
 */
FString FAnimationUtils::GetAnimationKeyFormatString(AnimationKeyFormat InFormat)
{
	switch(InFormat)
	{
	case AKF_ConstantKeyLerp:
		return FString("AKF_ConstantKeyLerp");
	case AKF_VariableKeyLerp:
		return FString("AKF_VariableKeyLerp");
	case AKF_PerTrackCompression:
		return FString("AKF_PerTrackCompression");
	default:
		UE_LOGF(LogAnimationCompression, Warning, "AnimationKeyFormat was not found:  %i", static_cast<int32>(InFormat) );
	}

	return FString("Unknown");
}


/**
 * Computes the 'height' of each track, relative to a given animation linkup.
 *
 * The track height is defined as the minimal number of bones away from an end effector (end effectors are 0, their parents are 1, etc...)
 *
  * @param BoneData				The bone data to check
 * @param NumTracks				The number of tracks
 * @param TrackHeights [OUT]	The computed track heights
 *
 */
#if WITH_EDITOR
void FAnimationUtils::CalculateTrackHeights(const FCompressibleAnimData& CompressibleAnimData, int32 NumTracks, TArray<int32>& TrackHeights)
{
	TrackHeights.Empty();
	TrackHeights.AddUninitialized(NumTracks);
	for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
	{
		TrackHeights[TrackIndex] = 0;
	}

	const TArray<FBoneData>& BoneData = CompressibleAnimData.BoneData;

	// Populate the bone 'height' table (distance from closest end effector, with 0 indicating an end effector)
	// setup the raw bone transformation and find all end effectors
	for (int32 BoneIndex = 0; BoneIndex < BoneData.Num(); ++BoneIndex)
	{
		// also record all end-effectors we find
		const FBoneData& Bone = BoneData[BoneIndex];
		if (Bone.IsEndEffector())
		{
			const FBoneData& EffectorBoneData = BoneData[BoneIndex];

			for (int32 FamilyIndex = 0; FamilyIndex < EffectorBoneData.BonesToRoot.Num(); ++FamilyIndex)
			{
				const int32 NextParentBoneIndex = EffectorBoneData.BonesToRoot[FamilyIndex];
				const int32 NextParentTrackIndex = GetAnimTrackIndexForSkeletonBone(NextParentBoneIndex, CompressibleAnimData.TrackToSkeletonMapTable);
				if (NextParentTrackIndex != INDEX_NONE)
				{
					const int32 CurHeight = TrackHeights[NextParentTrackIndex];
					TrackHeights[NextParentTrackIndex] = (CurHeight > 0) ? FMath::Min<int32>(CurHeight, (FamilyIndex+1)) : (FamilyIndex+1);
				}
			}
		}
	}
}
#endif // WITH_EDITOR

/**
 * Checks a set of key times to see if the spacing is uniform or non-uniform.
 * Note: If there are as many times as frames, they are automatically assumed to be uniformly spaced.
 * Note: If there are two or fewer times, they are automatically assumed to be uniformly spaced.
 *
 * @param AnimSeq		The animation sequence the Times array is associated with
 * @param Times			The array of key times
 *
 * @return				true if the keys are uniformly spaced (or one of the trivial conditions is detected).  false if any key spacing is greater than 1e-4 off.
 */
bool FAnimationUtils::HasUniformKeySpacing(int32 NumFrames, const TArray<float>& Times)
{
	if ((Times.Num() <= 2) || (Times.Num() == NumFrames))
	{
		return true;
	}

	float FirstDelta = Times[1] - Times[0];
	for (int32 i = 2; i < Times.Num(); ++i)
	{
		float DeltaTime = Times[i] - Times[i-1];

		if (fabs(DeltaTime - FirstDelta) > UE_KINDA_SMALL_NUMBER)
		{
			return false;
		}
	}

	return false;
}

namespace EPerturbationErrorMode
{
	enum Type
	{
		Transform,
		Rotation,
		Scale,
	};
}

template<int32>
FORCEINLINE void CalcErrorValues(FAnimPerturbationError& TrackError, const FTransform& RawTransform, const FTransform& NewTransform);

template<>
FORCEINLINE void CalcErrorValues<EPerturbationErrorMode::Transform>(FAnimPerturbationError& TrackError, const FTransform& RawTransform, const FTransform& NewTransform)
{
	TrackError.MaxErrorInTransDueToTrans = FMath::Max(TrackError.MaxErrorInTransDueToTrans, (RawTransform.GetLocation() - NewTransform.GetLocation()).SizeSquared());
	TrackError.MaxErrorInRotDueToTrans = FMath::Max(TrackError.MaxErrorInRotDueToTrans, FQuat::ErrorAutoNormalize(RawTransform.GetRotation(), NewTransform.GetRotation()));
	//TrackError.MaxErrorInScaleDueToTrans = FMath::Max(TrackError.MaxErrorInScaleDueToTrans, (RawTransform.GetScale3D() - NewTransform.GetScale3D()).SizeSquared());
}

template<>
FORCEINLINE void CalcErrorValues<EPerturbationErrorMode::Rotation>(FAnimPerturbationError& TrackError, const FTransform& RawTransform, const FTransform& NewTransform)
{
	TrackError.MaxErrorInTransDueToRot = FMath::Max(TrackError.MaxErrorInTransDueToRot, (RawTransform.GetLocation() - NewTransform.GetLocation()).SizeSquared());
	TrackError.MaxErrorInRotDueToRot = FMath::Max(TrackError.MaxErrorInRotDueToRot, FQuat::ErrorAutoNormalize(RawTransform.GetRotation(), NewTransform.GetRotation()));
	//TrackError.MaxErrorInScaleDueToRot = FMath::Max(TrackError.MaxErrorInScaleDueToRot, (RawTransform.GetScale3D() - NewTransform.GetScale3D()).SizeSquared());
}

template<>
FORCEINLINE void CalcErrorValues<EPerturbationErrorMode::Scale>(FAnimPerturbationError& TrackError, const FTransform& RawTransform, const FTransform& NewTransform)
{
	/*TrackError.MaxErrorInTransDueToScale = FMath::Max(TrackError.MaxErrorInTransDueToScale, (RawTransform.GetLocation() - NewTransform.GetLocation()).SizeSquared());
	TrackError.MaxErrorInRotDueToScale = FMath::Max(TrackError.MaxErrorInRotDueToScale, FQuat::ErrorAutoNormalize(RawTransform.GetRotation(), NewTransform.GetRotation()));
	TrackError.MaxErrorInScaleDueToScale = FMath::Max(TrackError.MaxErrorInScaleDueToScale, (RawTransform.GetScale3D() - NewTransform.GetScale3D()).SizeSquared());*/

	TrackError.MaxErrorInTransDueToScale = TrackError.MaxErrorInTransDueToRot; //Original tally code was calculating scale errors then using rot regardless.
	TrackError.MaxErrorInRotDueToScale = TrackError.MaxErrorInRotDueToRot;     
	//TrackError.MaxErrorInScaleDueToScale = TrackError.MaxErrorInScaleDueToRot;
}

struct FBoneTestItem
{
	int32 BoneIndex;
	bool  bIsEndEffector;
	FBoneTestItem(int32 InBoneIndex, const TArray<FBoneData>& BoneData)
		: BoneIndex(InBoneIndex)
		, bIsEndEffector(BoneData[BoneIndex].IsEndEffector())
	{}
};

#if WITH_EDITOR
template<int32 PERTURBATION_ERROR_MODE>
void CalcErrorsLoop(const TArray<FBoneTestItem>& BonesToTest, const FCompressibleAnimData& CompressibleAnimData, const TArray<FTransform>& RawAtoms, const TArray<FTransform>& RawTransforms, TArray<FTransform>& NewTransforms, FAnimPerturbationError& ThisBoneError)
{
	for (const FBoneTestItem& BoneTest : BonesToTest)
	{
		const int32 ParentIndex = CompressibleAnimData.RefSkeleton.GetParentIndex(BoneTest.BoneIndex);
		NewTransforms[BoneTest.BoneIndex] = RawAtoms[BoneTest.BoneIndex] * NewTransforms[ParentIndex];

		if (BoneTest.bIsEndEffector)
		{
			CalcErrorValues<PERTURBATION_ERROR_MODE>(ThisBoneError, RawTransforms[BoneTest.BoneIndex], NewTransforms[BoneTest.BoneIndex]);
		}
	}
}

/**
 * Perturbs the bone(s) associated with each track in turn, measuring the maximum error introduced in end effectors as a result
 */
void FAnimationUtils::TallyErrorsFromPerturbation(
	const FCompressibleAnimData& CompressibleAnimData,
	int32 NumTracks,
	const FVector& PositionNudge,
	const FQuat& RotationNudge,
	const FVector& ScaleNudge,
	TArray<FAnimPerturbationError>& InducedErrors)
{
	const int32 NumBones = CompressibleAnimData.BoneData.Num();

	const TArray<FTransform>& RefPose = CompressibleAnimData.RefLocalPoses;

	TArray<FTransform> RawAtoms;
	TArray<FTransform> RawTransforms;
	TArray<FTransform> NewTransforms;

	RawAtoms.AddZeroed(NumBones);
	RawTransforms.AddZeroed(NumBones);
	NewTransforms.AddZeroed(NumBones);

	InducedErrors.AddZeroed(NumTracks);

	FTransform Perturbation(RotationNudge, PositionNudge, ScaleNudge);

	// Build Track bone mapping for processingh
	struct FTrackBoneMapping
	{
		int32 TrackIndex;
		int32 BoneIndex;
		bool  bIsEndEffector;
		FTrackBoneMapping(int32 InTrackIndex, int32 InBoneIndex, const TArray<FBoneData>& BoneData)
			: TrackIndex(InTrackIndex)
			, BoneIndex(InBoneIndex)
			, bIsEndEffector(BoneData[BoneIndex].IsEndEffector())
		{}
	};

	TArray<FTrackBoneMapping> TracksAndBonesToTest;
	TracksAndBonesToTest.Reserve(NumTracks);

	for (int32 TrackToTest = 0; TrackToTest < NumTracks; TrackToTest++)
	{
		const int32 BoneIndex = CompressibleAnimData.TrackToSkeletonMapTable[TrackToTest].BoneTreeIndex;
		if (BoneIndex != INDEX_NONE)
		{
			TracksAndBonesToTest.Emplace(TrackToTest, BoneIndex, CompressibleAnimData.BoneData);
		}
	}

	Algo::SortBy(TracksAndBonesToTest, &FTrackBoneMapping::BoneIndex);

	// Prebuild bone test paths
	TArray<TArray<FBoneTestItem>> BonesToTestMap;
	BonesToTestMap.AddDefaulted(NumTracks);

	TArray<int32> TempBonesToTest;
	TempBonesToTest.Reserve(NumBones);

	for (FTrackBoneMapping& TrackBonePair : TracksAndBonesToTest)
	{
		TempBonesToTest.Reset();
		TempBonesToTest.Append(CompressibleAnimData.BoneData[TrackBonePair.BoneIndex].Children);

		int32 BoneToTestIndex = 0;
		while (BoneToTestIndex < TempBonesToTest.Num())
		{
			const int32 ThisBoneIndex = TempBonesToTest[BoneToTestIndex++];
			TempBonesToTest.Append(CompressibleAnimData.BoneData[ThisBoneIndex].Children);
		}

		TArray<FBoneTestItem>& BonesToTest = BonesToTestMap[TrackBonePair.TrackIndex];
		BonesToTest.Reserve(TempBonesToTest.Num());

		for (const int32 BoneToTest : TempBonesToTest)
		{
			BonesToTest.Emplace(BoneToTest, CompressibleAnimData.BoneData);
		}
	}


	for (int32 KeyIndex = 0; KeyIndex < CompressibleAnimData.NumberOfKeys; ++KeyIndex)
	{
		//Build Locals For Frame
		if (CompressibleAnimData.IsCancelled())
		{
			return;
		}
		RawAtoms = RefPose;
		for (const FTrackBoneMapping& TrackAndBone : TracksAndBonesToTest)
		{
			const FRawAnimSequenceTrack& RawTrack = CompressibleAnimData.RawAnimationData[TrackAndBone.TrackIndex];
			ExtractTransformForFrameFromTrackSafe(RawTrack, KeyIndex, RawAtoms[TrackAndBone.BoneIndex]);
		}

		//Build Reference Component Space for Frame
		RawTransforms[0] = RawAtoms[0];
		for (int32 BoneIndex = 1; BoneIndex < NumBones; ++BoneIndex)
		{
			const int32 ParentIndex = CompressibleAnimData.RefSkeleton.GetParentIndex(BoneIndex);
			RawTransforms[BoneIndex] = RawAtoms[BoneIndex] * RawTransforms[ParentIndex];
		}

		//For each track
		int32 PreviousCopyPoint = 0;

		for (const FTrackBoneMapping& TrackBonePair : TracksAndBonesToTest)
		{
			const int32 BoneToModify = TrackBonePair.BoneIndex;

			TArray<FBoneTestItem>& BonesToTest = BonesToTestMap[TrackBonePair.TrackIndex];

			FAnimPerturbationError& ThisBoneError = InducedErrors[TrackBonePair.TrackIndex];

			// Init unchanged cache
			for (int32 BoneIndex = PreviousCopyPoint; BoneIndex < BoneToModify; ++BoneIndex)
			{
				NewTransforms[BoneIndex] = RawTransforms[BoneIndex];
			}

			// Modify test bone
			PreviousCopyPoint = BoneToModify;
			
			// Calc Transform Error
			NewTransforms[BoneToModify] = RawAtoms[BoneToModify];
			NewTransforms[BoneToModify].AddToTranslation(PositionNudge);

			const int32 ModifiedParentIndex = CompressibleAnimData.RefSkeleton.GetParentIndex(BoneToModify);

			if (ModifiedParentIndex != INDEX_NONE) // Put Modified Bone in Component space
			{
				NewTransforms[BoneToModify] = NewTransforms[BoneToModify] * NewTransforms[ModifiedParentIndex];
			}

			if (TrackBonePair.bIsEndEffector)
			{
				CalcErrorValues<EPerturbationErrorMode::Transform>(ThisBoneError, RawTransforms[BoneToModify], NewTransforms[BoneToModify]);
			}
			CalcErrorsLoop<EPerturbationErrorMode::Transform>(BonesToTest, CompressibleAnimData, RawAtoms, RawTransforms, NewTransforms, ThisBoneError);

			// Calc Rotatin Error
			NewTransforms[BoneToModify] = RawAtoms[BoneToModify];
			FQuat NewR = RawAtoms[BoneToModify].GetRotation();
			NewR += RotationNudge;
			NewR.Normalize();
			NewTransforms[BoneToModify].SetRotation(NewR);

			if (ModifiedParentIndex != INDEX_NONE) // Put Modified Bone in Component space
			{
				NewTransforms[BoneToModify] = NewTransforms[BoneToModify] * NewTransforms[ModifiedParentIndex];
			}

			if (TrackBonePair.bIsEndEffector)
			{
				CalcErrorValues<EPerturbationErrorMode::Rotation>(ThisBoneError, RawTransforms[BoneToModify], NewTransforms[BoneToModify]);
			}
			CalcErrorsLoop<EPerturbationErrorMode::Rotation>(BonesToTest, CompressibleAnimData, RawAtoms, RawTransforms, NewTransforms, ThisBoneError);

			//Calc Scale Error
			NewTransforms[BoneToModify] = RawAtoms[BoneToModify];
			NewTransforms[BoneToModify].SetScale3D(RawAtoms[BoneToModify].GetScale3D() + ScaleNudge);

			if (ModifiedParentIndex != INDEX_NONE) // Put Modified Bone in Component space
			{
				NewTransforms[BoneToModify] = NewTransforms[BoneToModify] * NewTransforms[ModifiedParentIndex];
			}

			if (TrackBonePair.bIsEndEffector)
			{
				CalcErrorValues<EPerturbationErrorMode::Scale>(ThisBoneError, RawTransforms[BoneToModify], NewTransforms[BoneToModify]);
			}
			CalcErrorsLoop<EPerturbationErrorMode::Scale>(BonesToTest, CompressibleAnimData, RawAtoms, RawTransforms, NewTransforms, ThisBoneError);
		}
	}

	for (FAnimPerturbationError& Error : InducedErrors)
	{
		Error.MaxErrorInTransDueToTrans = FMath::Sqrt(Error.MaxErrorInTransDueToTrans);
		Error.MaxErrorInTransDueToRot = FMath::Sqrt(Error.MaxErrorInTransDueToRot);
		Error.MaxErrorInTransDueToScale = FMath::Sqrt(Error.MaxErrorInTransDueToScale);
		/*Error.MaxErrorInScaleDueToTrans = FMath::Sqrt(Error.MaxErrorInScaleDueToTrans); //Not used by compression code TODO: Investigate and either use or remove
		Error.MaxErrorInScaleDueToRot = FMath::Sqrt(Error.MaxErrorInScaleDueToRot);
		Error.MaxErrorInScaleDueToScale = FMath::Sqrt(Error.MaxErrorInScaleDueToScale);*/
	}
}
#endif // WITH_EDITOR

static UAnimBoneCompressionSettings* DefaultBoneCompressionSettings = nullptr;
static UAnimBoneCompressionSettings* DefaultRecorderBoneCompressionSettings = nullptr;
static UAnimCurveCompressionSettings* DefaultCurveCompressionSettings = nullptr;
static UVariableFrameStrippingSettings* DefaultVariableFrameStrippingSettings = nullptr;

static void EnsureDependenciesAreLoaded(UObject* Object)
{
	if (Object == nullptr)
	{
		return;	// Nothing to do
	}

	if (Object->HasAnyFlags(RF_NeedLoad))
	{
		UE_LOGF(LogAnimationCompression, Warning, "EnsureDependenciesAreLoaded: [Preload] %ls", *Object->GetPathName());
	}
	
	// Always call conditional preload as the test is a little bit more involved than the one we did for the warning above.
	Object->ConditionalPreload();

	if (Object->HasAnyFlags(RF_NeedPostLoad))
	{
		UE_LOGF(LogAnimationCompression, Warning, "EnsureDependenciesAreLoaded: [ConditionalPostLoad] %ls", *Object->GetPathName());
		Object->ConditionalPostLoad();
	}

	TArray<UObject*> ObjectReferences;
	FReferenceFinder(ObjectReferences, Object, false, true, true, true).FindReferences(Object);

	for (UObject* Dependency : ObjectReferences)
	{
		if (Dependency->HasAnyFlags(RF_NeedLoad))
		{
			UE_LOGF(LogAnimationCompression, Warning, "EnsureDependenciesAreLoaded: [Preload Dependency] %ls", *Dependency->GetPathName());
		}

		// Always call conditional preload as the test is a little bit more involved than the one we did for the warning above.
		Dependency->ConditionalPreload();

		if (Dependency->HasAnyFlags(RF_NeedPostLoad))
		{
			UE_LOGF(LogAnimationCompression, Warning, "EnsureDependenciesAreLoaded: [ConditionalPostLoad Dependency] %ls", *Dependency->GetPathName());
			Dependency->ConditionalPostLoad();
		}
	}
}



UObject* GetDefaultAnimationCompressionSettings(const TCHAR* IniValueName, bool bIsFatal)
{
	const FConfigSection* AnimDefaultObjectSettingsSection = GConfig->GetSection(TEXT("Animation.DefaultObjectSettings"), false, GEngineIni);
	const FConfigValue* Value = AnimDefaultObjectSettingsSection != nullptr ? AnimDefaultObjectSettingsSection->Find(IniValueName) : nullptr;

	if (Value == nullptr)
	{
		if (bIsFatal)
		{
			UE_LOGF(LogAnimationCompression, Fatal, "Couldn't find default compression setting for '%ls' under '[Animation.DefaultObjectSettings]'", IniValueName);
		}
		else
		{
			UE_LOGF(LogAnimationCompression, Warning, "Couldn't find default compression setting for '%ls' under '[Animation.DefaultObjectSettings]'", IniValueName);
		}

		return nullptr;
	}

	const FString& CompressionSettingsName = Value->GetValue();
	UObject* DefaultCompressionSettings = LoadObject<UObject>(nullptr, *CompressionSettingsName);

	if (DefaultCompressionSettings == nullptr)
	{
		if (bIsFatal)
		{
			UE_LOGF(LogAnimationCompression, Fatal, "Couldn't load default compression settings asset with path '%ls'", *CompressionSettingsName);
		}
		else
		{
			UE_LOGF(LogAnimationCompression, Warning, "Couldn't load default compression settings asset with path '%ls'", *CompressionSettingsName);
		}

		return nullptr;
	}

	// Force load the default settings and all its dependencies just in case it hasn't happened yet
	EnsureDependenciesAreLoaded(DefaultCompressionSettings);

	DefaultCompressionSettings->AddToRoot();

	return DefaultCompressionSettings;
}

#if WITH_EDITOR

void FAnimationUtils::PreloadCompressionSettings()
{
	GetDefaultAnimationBoneCompressionSettings();
	GetDefaultAnimationRecorderBoneCompressionSettings();
	GetDefaultAnimationCurveCompressionSettings();
	GetDefaultVariableFrameStrippingSettings();
}

#endif

UAnimBoneCompressionSettings* FAnimationUtils::GetDefaultAnimationBoneCompressionSettings()
{
	if (DefaultBoneCompressionSettings == nullptr)
	{
		DefaultBoneCompressionSettings = Cast<UAnimBoneCompressionSettings>(GetDefaultAnimationCompressionSettings(TEXT("BoneCompressionSettings"), false));

		if (DefaultBoneCompressionSettings == nullptr)
		{
			DefaultBoneCompressionSettings = Cast<UAnimBoneCompressionSettings>(GetDefaultAnimationCompressionSettings(TEXT("BoneCompressionSettingsFallback"), true));
		}
	}

	return DefaultBoneCompressionSettings;
}

UAnimBoneCompressionSettings* FAnimationUtils::GetDefaultAnimationRecorderBoneCompressionSettings()
{
	if (DefaultRecorderBoneCompressionSettings == nullptr)
	{
		DefaultRecorderBoneCompressionSettings = Cast<UAnimBoneCompressionSettings>(GetDefaultAnimationCompressionSettings(TEXT("AnimationRecorderBoneCompressionSettings"), false));

		if (DefaultRecorderBoneCompressionSettings == nullptr)
		{
			DefaultRecorderBoneCompressionSettings = Cast<UAnimBoneCompressionSettings>(GetDefaultAnimationCompressionSettings(TEXT("AnimationRecorderBoneCompressionSettingsFallback"), true));
		}
	}

	return DefaultRecorderBoneCompressionSettings;
}

UAnimCurveCompressionSettings* FAnimationUtils::GetDefaultAnimationCurveCompressionSettings()
{
	if (DefaultCurveCompressionSettings == nullptr)
	{
		DefaultCurveCompressionSettings = Cast<UAnimCurveCompressionSettings>(GetDefaultAnimationCompressionSettings(TEXT("CurveCompressionSettings"), false));

		if (DefaultCurveCompressionSettings == nullptr)
		{
			DefaultCurveCompressionSettings = Cast<UAnimCurveCompressionSettings>(GetDefaultAnimationCompressionSettings(TEXT("CurveCompressionSettingsFallback"), true));
		}
	}

	return DefaultCurveCompressionSettings;
}

UVariableFrameStrippingSettings* FAnimationUtils::GetDefaultVariableFrameStrippingSettings()
{
	if (DefaultVariableFrameStrippingSettings == nullptr)
	{
		DefaultVariableFrameStrippingSettings = Cast<UVariableFrameStrippingSettings>(GetDefaultAnimationCompressionSettings(TEXT("VariableFrameStrippingSettings"), true));
	}

	return DefaultVariableFrameStrippingSettings;
}

void FAnimationUtils::EnsureAnimSequenceLoaded(UAnimSequence& AnimSeq)
{
	EnsureDependenciesAreLoaded(&AnimSeq);
	EnsureDependenciesAreLoaded(AnimSeq.GetSkeleton());
	EnsureDependenciesAreLoaded(AnimSeq.BoneCompressionSettings);
	EnsureDependenciesAreLoaded(AnimSeq.CurveCompressionSettings);
	EnsureDependenciesAreLoaded(AnimSeq.RefPoseSeq);
	EnsureDependenciesAreLoaded(AnimSeq.VariableFrameStrippingSettings);
}

void FAnimationUtils::ExtractTransformForFrameFromTrackSafe(const FRawAnimSequenceTrack& RawTrack, int32 Frame, FTransform& OutAtom)
{
	// Bail out (with rather wacky data) if data is empty for some reason.
	if (RawTrack.PosKeys.Num() == 0 || RawTrack.RotKeys.Num() == 0)
	{
		OutAtom.SetIdentity();
		return;
	}

	ExtractTransformForFrameFromTrack(RawTrack, Frame, OutAtom);
}

void FAnimationUtils::ExtractTransformForFrameFromTrack(const FRawAnimSequenceTrack& RawTrack, int32 Frame, FTransform& OutAtom)
{
	static const FVector DefaultScale3D = FVector(1.f);

	const int32 PosKeyIndex1 = FMath::Min(Frame, RawTrack.PosKeys.Num() - 1);
	const int32 RotKeyIndex1 = FMath::Min(Frame, RawTrack.RotKeys.Num() - 1);

	if (RawTrack.ScaleKeys.Num() > 0)
	{
		const int32 ScaleKeyIndex1 = FMath::Min(Frame, RawTrack.ScaleKeys.Num() - 1);
		OutAtom = FTransform(FQuat(RawTrack.RotKeys[RotKeyIndex1]), FVector(RawTrack.PosKeys[PosKeyIndex1]), FVector(RawTrack.ScaleKeys[ScaleKeyIndex1]));
	}
	else
	{
		OutAtom = FTransform(FQuat(RawTrack.RotKeys[RotKeyIndex1]), FVector(RawTrack.PosKeys[PosKeyIndex1]), FVector(DefaultScale3D));
	}
}

void FAnimationUtils::ExtractTransformFromTrack(float Time, int32 NumFrames, float SequenceLength, const FRawAnimSequenceTrack& RawTrack, EAnimInterpolationType Interpolation, FTransform &OutAtom)
{
	ExtractTransformFromTrack(RawTrack, static_cast<double>(Time), NumFrames, static_cast<double>(SequenceLength), Interpolation, OutAtom);
}

void FAnimationUtils::ExtractTransformFromTrack(const FRawAnimSequenceTrack& RawTrack, double Time, int32 NumFrames, double SequenceLength, EAnimInterpolationType Interpolation, FTransform &OutAtom)
{
	// Bail out (with rather wacky data) if data is empty for some reason.
	if (RawTrack.PosKeys.Num() == 0 || RawTrack.RotKeys.Num() == 0)
	{
		OutAtom.SetIdentity();
		return;
	}

	int32 KeyIndex1, KeyIndex2;
	float Alpha;
	FAnimationRuntime::GetKeyIndicesFromTime(KeyIndex1, KeyIndex2, Alpha, Time, NumFrames, SequenceLength);
	// @Todo fix me: this change is not good, it has lots of branches. But we'd like to save memory for not saving scale if no scale change exists
	
	static const FVector DefaultScale3D = FVector(1.f);

	if (Interpolation == EAnimInterpolationType::Step)
	{
		Alpha = 0.f;
	}

	if (Alpha <= 0.f)
	{
		ExtractTransformForFrameFromTrack(RawTrack, KeyIndex1, OutAtom);
		return;
	}
	else if (Alpha >= 1.f)
	{
		ExtractTransformForFrameFromTrack(RawTrack, KeyIndex2, OutAtom);
		return;
	}

	const int32 PosKeyIndex1 = FMath::Min(KeyIndex1, RawTrack.PosKeys.Num() - 1);
	const int32 RotKeyIndex1 = FMath::Min(KeyIndex1, RawTrack.RotKeys.Num() - 1);

	const int32 PosKeyIndex2 = FMath::Min(KeyIndex2, RawTrack.PosKeys.Num() - 1);
	const int32 RotKeyIndex2 = FMath::Min(KeyIndex2, RawTrack.RotKeys.Num() - 1);

	FTransform KeyAtom1, KeyAtom2;

	const bool bHasScaleKey = (RawTrack.ScaleKeys.Num() > 0);
	if (bHasScaleKey)
	{
		const int32 ScaleKeyIndex1 = FMath::Min(KeyIndex1, RawTrack.ScaleKeys.Num() - 1);
		const int32 ScaleKeyIndex2 = FMath::Min(KeyIndex2, RawTrack.ScaleKeys.Num() - 1);

		KeyAtom1 = FTransform(FQuat(RawTrack.RotKeys[RotKeyIndex1]), FVector(RawTrack.PosKeys[PosKeyIndex1]), FVector(RawTrack.ScaleKeys[ScaleKeyIndex1]));
		KeyAtom2 = FTransform(FQuat(RawTrack.RotKeys[RotKeyIndex2]), FVector(RawTrack.PosKeys[PosKeyIndex2]), FVector(RawTrack.ScaleKeys[ScaleKeyIndex2]));
	}
	else
	{
		KeyAtom1 = FTransform(FQuat(RawTrack.RotKeys[RotKeyIndex1]), FVector(RawTrack.PosKeys[PosKeyIndex1]), DefaultScale3D);
		KeyAtom2 = FTransform(FQuat(RawTrack.RotKeys[RotKeyIndex2]), FVector(RawTrack.PosKeys[PosKeyIndex2]), DefaultScale3D);
	}

	// 	UE_LOGF(LogAnimation, Log, " *  *  *  Position. PosKeyIndex1: %3d, PosKeyIndex2: %3d, Alpha: %f", PosKeyIndex1, PosKeyIndex2, Alpha);
	// 	UE_LOGF(LogAnimation, Log, " *  *  *  Rotation. RotKeyIndex1: %3d, RotKeyIndex2: %3d, Alpha: %f", RotKeyIndex1, RotKeyIndex2, Alpha);

	// Ensure rotations are normalized (Added for Jira UE-53971)
	KeyAtom1.NormalizeRotation();
	KeyAtom2.NormalizeRotation();

	OutAtom.Blend(KeyAtom1, KeyAtom2, Alpha);
	OutAtom.NormalizeRotation();
}

#if WITH_EDITOR
void FAnimationUtils::ExtractTransformFromCompressionData(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& CompressedAnimData, float Time, int32 TrackIndex, bool bUseRawData, FTransform& OutBoneTransform)
{
	ExtractTransformFromCompressionData(CompressibleAnimData, CompressedAnimData, (double)Time, TrackIndex, bUseRawData, OutBoneTransform);
}

void FAnimationUtils::ExtractTransformFromCompressionData(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& CompressedAnimData, double Time, int32 TrackIndex, bool bUseRawData, FTransform& OutBoneTransform)
{
	FUECompressedAnimDataMutable& AnimDataMutable = static_cast<FUECompressedAnimDataMutable&>(*CompressedAnimData.AnimData);

	// If the caller didn't request that raw animation data be used . . .
	if (!bUseRawData && AnimDataMutable.IsValid())
	{
		// Build our read-only version from the mutable source
		FUECompressedAnimData AnimData(AnimDataMutable);

		FAnimSequenceDecompressionContext DecompContext(CompressibleAnimData.SampledFrameRate, CompressibleAnimData.GetNumberOfFrames(), CompressibleAnimData.Interpolation, CompressibleAnimData.AnimFName, AnimData, CompressibleAnimData.RefLocalPoses, CompressibleAnimData.TrackToSkeletonMapTable, nullptr, CompressibleAnimData.bIsValidAdditive, CompressibleAnimData.AdditiveType);
		DecompContext.Seek(Time);
		CompressedAnimData.Codec->DecompressBone(DecompContext, TrackIndex, OutBoneTransform);
		return;
	}

	FAnimationUtils::ExtractTransformFromTrack(CompressibleAnimData.RawAnimationData[TrackIndex], Time, CompressibleAnimData.NumberOfKeys, CompressibleAnimData.SequenceLength, CompressibleAnimData.Interpolation, OutBoneTransform);
}

bool FAnimationUtils::CompressAnimBones(FCompressibleAnimData& AnimSeq, FCompressibleAnimDataResult& Target)
{
	// Clear any previous data we might have even if we end up failing to compress
	Target = FCompressibleAnimDataResult();

	if (AnimSeq.BoneCompressionSettings == nullptr || !AnimSeq.BoneCompressionSettings->AreSettingsValid())
	{
		return false;
	}

	return AnimSeq.BoneCompressionSettings->Compress(AnimSeq, Target);
}

bool FAnimationUtils::CompressAnimCurves(FCompressibleAnimData& AnimSeq, FCompressedAnimSequence& Target)
{
	if (AnimSeq.CurveCompressionSettings == nullptr || !AnimSeq.CurveCompressionSettings->AreSettingsValid())
	{
		return false;
	}

	return AnimSeq.CurveCompressionSettings->Compress(AnimSeq, Target);
}
#endif
