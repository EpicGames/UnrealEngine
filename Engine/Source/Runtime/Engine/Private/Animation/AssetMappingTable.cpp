// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UAssetMappingTable.cpp: AssetMappingTable functionality for sharing animations
=============================================================================*/ 

#include "Animation/AssetMappingTable.h"
#include "Animation/AnimationAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetMappingTable)

//@todo should move all this window stuff somewhere else. Persona?

#define LOCTEXT_NAMESPACE "AssetMappingTable"

bool FAssetMapping::IsValidMapping() const
{
	return IsValidMapping(SourceAsset, TargetAsset);
}

bool FAssetMapping::IsValidMapping(UAnimationAsset* InSourceAsset, UAnimationAsset* InTargetAsset) 
{
	// for now we only allow same class
	return ( InSourceAsset && InTargetAsset && InSourceAsset != InTargetAsset && 
			InSourceAsset->GetClass() == InTargetAsset->GetClass() &&
			InSourceAsset->GetSkeleton() == InTargetAsset->GetSkeleton() &&
			InSourceAsset->IsValidAdditive() == InTargetAsset->IsValidAdditive()
			// @note check if same kind of additive?
		);
}

bool FAssetMapping::SetTargetAsset(UAnimationAsset* InTargetAsset)
{
	if (SourceAsset && InTargetAsset)
	{
		// if source and target is same, we clear target asset
		if (IsValidMapping(SourceAsset, InTargetAsset))
		{
			TargetAsset = InTargetAsset;
			return true;
		}
	}

	return false;
}


//////////////////////////////////////////////////////////////////////////
//
//////////////////////////////////////////////////////////////////////////
UAssetMappingTable::UAssetMappingTable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAssetMappingTable::Clear()
{
	MappedAssets.Empty();
}

int32 UAssetMappingTable::FindMappedAsset(const UAnimationAsset* NewAsset) const
{
	for (int32 ExistingId = 0; ExistingId < MappedAssets.Num(); ++ExistingId)
	{
		if (MappedAssets[ExistingId].SourceAsset == NewAsset)
		{
			// already exists
			return ExistingId;
		}
	}

	return INDEX_NONE;
}

// want to make sure if any asset not existing anymore gets removed, 
// if new assets will be added to the list

void UAssetMappingTable::RefreshAssetList(const TArray<UAnimationAsset*>& AnimAssets)
{
	// clear the list first, if source got disappeared. 
	RemovedUnusedSources();

	// now we have current existing list. Create bool buffer for if used or not
	TArray<bool> bUsedAssetList;
	bUsedAssetList.AddZeroed(MappedAssets.Num());

	for (UAnimationAsset* AnimAsset : AnimAssets)
	{
		int32 ExistingIndex = FindMappedAsset(AnimAsset);
		// make sure to remove unused assets
		if (ExistingIndex != INDEX_NONE)
		{
			// the new ones' won't exists here. Make sure you're in the valid range (old index)
			if (bUsedAssetList.IsValidIndex(ExistingIndex))
			{
				// if used, mark it
				bUsedAssetList[ExistingIndex] = true;
			}
		}
	}

	// we're going to remove unused items, so go from back
	// we only added so far, so the index shouldn't have changed
	for (int32 OldItemIndex = bUsedAssetList.Num() - 1; OldItemIndex >= 0; --OldItemIndex)
	{
		if (!bUsedAssetList[OldItemIndex])
		{
			Modify();
			MappedAssets.RemoveAt(OldItemIndex);
		}
	}
}

UAnimationAsset* UAssetMappingTable::GetMappedAsset(UAnimationAsset* SourceAsset) const
{
	if (SourceAsset)
	{
		int32 ExistingIndex = FindMappedAsset(SourceAsset);

		if (ExistingIndex != INDEX_NONE)
		{
			UAnimationAsset* TargetAsset = MappedAssets[ExistingIndex].TargetAsset;
			return (TargetAsset)? TargetAsset: SourceAsset;
		}
	}

	// if it's not mapped just send out SourceAsset
	return SourceAsset;
}

void UAssetMappingTable::RemovedUnusedSources()
{
	for (int32 ExistingId = 0; ExistingId < MappedAssets.Num(); ++ExistingId)
	{
		if (MappedAssets[ExistingId].IsValidMapping() == false)
		{
			Modify();
			MappedAssets.RemoveAt(ExistingId);
			--ExistingId;
		}
	}
}

bool UAssetMappingTable::RemapAsset(UAnimationAsset* SourceAsset, UAnimationAsset* TargetAsset)
{
	if (SourceAsset)
	{
		bool bValidAsset = FAssetMapping::IsValidMapping(SourceAsset, TargetAsset);
		int32 ExistingIndex = FindMappedAsset(SourceAsset);
		if (bValidAsset)
		{
			Modify();
			if (ExistingIndex == INDEX_NONE)
			{
				ExistingIndex = MappedAssets.Add(FAssetMapping(SourceAsset));
			}

			return MappedAssets[ExistingIndex].SetTargetAsset(TargetAsset);
		}
		else if (ExistingIndex != INDEX_NONE)
		{
			Modify();
			MappedAssets.RemoveAtSwap(ExistingIndex);
		}

		return true;
	}

	return false;
}
#if WITH_EDITOR
bool UAssetMappingTable::GetAllAnimationSequencesReferred(TArray<class UAnimationAsset*>& AnimationSequences, bool bRecursive /*= true*/)
{
	for (FAssetMapping& AssetMapping : MappedAssets)
	{
		UAnimationAsset* AnimAsset = AssetMapping.SourceAsset;
		if (AnimAsset)
		{
			AnimAsset->HandleAnimReferenceCollection(AnimationSequences, bRecursive);
		}

		AnimAsset = AssetMapping.TargetAsset;
		if (AnimAsset)
		{
			AnimAsset->HandleAnimReferenceCollection(AnimationSequences, bRecursive);
		}
	}

	return (AnimationSequences.Num() > 0);
}

void UAssetMappingTable::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	for (FAssetMapping& AssetMapping : MappedAssets)
	{
		TObjectPtr<UAnimationAsset>& AnimAsset = AssetMapping.SourceAsset;
		if (AnimAsset)
		{
			// now fix everythign else
			UAnimationAsset* const* ReplacementAsset = ReplacementMap.Find(AnimAsset);
			if (ReplacementAsset)
			{
				AnimAsset = *ReplacementAsset;
			}
		}

		AnimAsset = AssetMapping.TargetAsset;
		if (AnimAsset)
		{
			// now fix everythign else
			UAnimationAsset* const* ReplacementAsset = ReplacementMap.Find(AnimAsset);
			if (ReplacementAsset)
			{
				AnimAsset = *ReplacementAsset;
			}
		}
	}
}
#endif // WITH_EDITOR
#undef LOCTEXT_NAMESPACE 

