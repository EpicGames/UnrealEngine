// Copyright Epic Games, Inc. All Rights Reserved.


#include "Animation/AttributeBlendData.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AttributesRuntime.h"

namespace UE { namespace Anim {

	FAttributeBlendData::FAttributeBlendData() : UniformWeight(0.f), AdditiveType(AAT_None) {}

	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, const UScriptStruct* AttributeScriptStruct) : UniformWeight(0.f), AdditiveType(AAT_None)
	{
		Weights = SourceWeights;
		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}
	}

	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const TArrayView<const float> SourceWeights, const UScriptStruct* AttributeScriptStruct) : UniformWeight(0.f), AdditiveType(AAT_None)
	{
		Weights = SourceWeights;
		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = *SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}
	}

	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const float InUniformWeight, const UScriptStruct* AttributeScriptStruct) : UniformWeight(InUniformWeight), AdditiveType(AAT_None)
	{
		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = *SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}
	}

	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const float InUniformWeight, EAdditiveAnimationType InAdditiveType, const UScriptStruct* AttributeScriptStruct) : UniformWeight(InUniformWeight), AdditiveType(InAdditiveType)
	{
		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = *SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}
	}
	
	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const TMap<UE::Anim::FAttributeId, float>& InAttributeWeightOverrides,  const float InUniformWeight, EAdditiveAnimationType InAdditiveType, const UScriptStruct* AttributeScriptStruct) : UniformWeight(InUniformWeight), AdditiveType(InAdditiveType)
	{
		AttributeWeightOverrides = InAttributeWeightOverrides;
	
		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = *SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}
	}

	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const FPerBoneBlendWeight> InPerBoneBlendWeights, const UScriptStruct* AttributeScriptStruct) : UniformWeight(0.f), AdditiveType(AAT_None)
	{
		PerBoneWeights = InPerBoneBlendWeights;

		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}

		HighestBoneWeightedIndices.SetNum(InPerBoneBlendWeights.Num());
		for (int32 BoneIndex = 0; BoneIndex < HighestBoneWeightedIndices.Num(); ++BoneIndex)
		{
			const FPerBoneBlendWeight& Weight = InPerBoneBlendWeights[BoneIndex];
			if (Weight.BlendWeight > .5f)
			{
				HighestBoneWeightedIndices[BoneIndex] = Weight.SourceIndex;
			}
			else
			{
				HighestBoneWeightedIndices[BoneIndex] = 0;
			}
		}
	}

	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> InPerBoneInterpolationIndices, const TArrayView<const FBlendSampleData> InBlendSampleDataCache, const UScriptStruct* AttributeScriptStruct) : UniformWeight(0.f), AdditiveType(AAT_None)
	{
		PerBoneInterpolationIndices = InPerBoneInterpolationIndices;
		BlendSampleDataCache = InBlendSampleDataCache;

		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}
	
		HighestBoneWeightedIndices.Reserve(InPerBoneInterpolationIndices.Num());
		for (int32 Index = 0; Index < InPerBoneInterpolationIndices.Num(); ++Index)
		{
			float Weight = -1.f;
			int32 HighestIndex = INDEX_NONE;
			for (int32 EntryIndex = 0; EntryIndex < SourceAttributes.Num(); ++EntryIndex)
			{
				const int32 BoneIndex = HighestBoneWeightedIndices.Num();
				const float BoneWeight = GetBoneWeight(EntryIndex, BoneIndex);
				if (BoneWeight > Weight)
				{
					Weight = BoneWeight;
					HighestIndex = EntryIndex;
				}
			}

			HighestBoneWeightedIndices.Add(HighestIndex);
		}
	}

	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> InPerBoneInterpolationIndices, const TArrayView<const FBlendSampleData> InBlendSampleDataCache, TArrayView<const int32> InBlendSampleDataCacheIndices, const UScriptStruct* AttributeScriptStruct) : UniformWeight(0.f), AdditiveType(AAT_None)
	{
		PerBoneInterpolationIndices = InPerBoneInterpolationIndices;
		BlendSampleDataCache = InBlendSampleDataCache;
		WeightIndices = InBlendSampleDataCacheIndices;

		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}

		HighestBoneWeightedIndices.Reserve(InPerBoneInterpolationIndices.Num());
		for (int32 Index = 0; Index < InPerBoneInterpolationIndices.Num(); ++Index)
		{
			float Weight = -1.f;
			int32 HighestIndex = INDEX_NONE;
			for (int32 EntryIndex = 0; EntryIndex < SourceAttributes.Num(); ++EntryIndex)
			{
				const int32 BoneIndex = HighestBoneWeightedIndices.Num();
				const float BoneWeight = GetBoneWeight(EntryIndex, BoneIndex);
				if (BoneWeight > Weight)
				{
					Weight = BoneWeight;
					HighestIndex = EntryIndex;
				}
			}

			HighestBoneWeightedIndices.Add(HighestIndex);
		}
	}

	FAttributeBlendData::FAttributeBlendData(const FStackAttributeContainer& BaseAttributes, const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const FPerBoneBlendWeight> InPerBoneBlendWeights, const UScriptStruct* AttributeScriptStruct) : UniformWeight(0.f), AdditiveType(AAT_None), bPerBoneFilter(true)
	{
		PerBoneWeights = InPerBoneBlendWeights;
		ProcessAttributes(BaseAttributes, 0, AttributeScriptStruct);

		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex + 1, AttributeScriptStruct);
		}

		HighestBoneWeightedIndices.Reserve(InPerBoneBlendWeights.Num());
		for (const FPerBoneBlendWeight& BoneWeight : InPerBoneBlendWeights)
		{
			// First input takes precedence with equal weighting
			if (BoneWeight.BlendWeight > 0.5f)
			{
				HighestBoneWeightedIndices.Add(BoneWeight.SourceIndex + 1);
			}
			else
			{
				HighestBoneWeightedIndices.Add(0);
			}
		}
	}

	FAttributeBlendData::FAttributeBlendData(const FStackAttributeContainer& SourceAttributes1, const FStackAttributeContainer& SourceAttributes2, const TArrayView<const float> WeightsOfSource2, const UScriptStruct* AttributeScriptStruct) : UniformWeight(0.f), AdditiveType(AAT_None)
	{
		Weights = WeightsOfSource2;

		HighestBoneWeightedIndices.Reserve(WeightsOfSource2.Num());
		for (const float& BoneWeight : WeightsOfSource2)
		{
			// First input takes precedence with equal weighting
			if (BoneWeight > 0.5f)
			{
				HighestBoneWeightedIndices.Add(1);
			}
			else
			{
				HighestBoneWeightedIndices.Add(0);
			}
		}

		ProcessAttributes(SourceAttributes1, 0, AttributeScriptStruct);
		ProcessAttributes(SourceAttributes2, 1, AttributeScriptStruct);
	}

	FAttributeBlendData::FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, const TArrayView<const int32> SourceWeightsIndices, const UScriptStruct* AttributeScriptStruct) : UniformWeight(0.f), AdditiveType(AAT_None)
	{
		Weights = SourceWeights;
		WeightIndices = SourceWeightsIndices;

		for (int32 SourceAttributesIndex = 0; SourceAttributesIndex < SourceAttributes.Num(); ++SourceAttributesIndex)
		{
			const FStackAttributeContainer& CustomAttributes = SourceAttributes[SourceAttributesIndex];
			ProcessAttributes(CustomAttributes, SourceAttributesIndex, AttributeScriptStruct);
		}
	}

	void FAttributeBlendData::ProcessAttributes(const FStackAttributeContainer& AttributeContainers, int32 SourceAttributesIndex, const UScriptStruct* AttributeScriptStruct)
	{
		auto FindExistingUniqueAttribute = [this](const FAttributeId& Identifier)
		{
			return UniqueAttributes.IndexOfByPredicate([Identifier](const FUniqueAttribute& Attribute)
			{
				return *Attribute.Identifier == Identifier;
			});
		};

		auto FindExistingAttributeSet = [this](const FAttributeId& Identifier)
		{
			return AttributeSets.IndexOfByPredicate([Identifier](const FAttributeSet& AttributeSet)
			{
				return *AttributeSet.Identifier == Identifier;
			});
		};

		auto AddAttributeToSet = [this](FAttributeSet& AttributeSet, const uint8* DataPtr, int32 WeightIndex)
		{
			AttributeSet.DataPtrs.Add(DataPtr);
			AttributeSet.WeightIndices.Add(WeightIndex);

			// Only track highest-weighted container when using container-level weights (not per-bone)
			if (HasContainerWeights())
			{
				const float Weight = GetContainerWeight(WeightIndex);
				if (Weight > AttributeSet.HighestWeight)
				{
					AttributeSet.HighestWeight = Weight;
					AttributeSet.HighestWeightedIndex = AttributeSet.DataPtrs.Num() - 1;
				}
			}
		};

		const int32 WeightIndex = SourceAttributesIndex;
		const int32 TypeIndex = AttributeContainers.FindTypeIndex(AttributeScriptStruct);
		if (TypeIndex != INDEX_NONE)
		{
			const TConstArrayView<TWrappedAttribute<FAnimStackAllocator>> ValuesArray = AttributeContainers.GetValues(TypeIndex);
			const TConstArrayView<FAttributeId> AttributeIdentifiers = AttributeContainers.GetKeys(TypeIndex);
			
			for (int32 AttributeIndex = 0; AttributeIndex < AttributeIdentifiers.Num(); ++AttributeIndex)
			{
				const FAttributeId& AttributeIdentifier = AttributeIdentifiers[AttributeIndex];
				const int32 ExistingAttributeSetIndex = FindExistingAttributeSet(AttributeIdentifier);

				if (ExistingAttributeSetIndex != INDEX_NONE)
				{
					// Add entry to the set
					FAttributeSet& ExistingSet = AttributeSets[ExistingAttributeSetIndex];
					AddAttributeToSet(ExistingSet, ValuesArray[AttributeIndex].GetPtr<uint8>(), WeightIndex);
				}
				else
				{
					const int32 ExistingUniqueAttributeIndex = FindExistingUniqueAttribute(AttributeIdentifier);
					if (ExistingUniqueAttributeIndex != INDEX_NONE)
					{
						// Need to create a set
						const FUniqueAttribute& ExistingUniqueAttribute = UniqueAttributes[ExistingUniqueAttributeIndex];

						FAttributeSet& AttributeSet = AttributeSets.AddZeroed_GetRef();
						AttributeSet.Identifier = &AttributeIdentifier;
						AttributeSet.HighestWeight = -1.f;
						AttributeSet.HighestWeightedIndex = -1;

						// Add existing data
						AddAttributeToSet(AttributeSet, ExistingUniqueAttribute.DataPtr, ExistingUniqueAttribute.WeightIndex);

						// Add new data
						AddAttributeToSet(AttributeSet, ValuesArray[AttributeIndex].GetPtr<uint8>(), WeightIndex);

						// Remove as a unique attribute
						UniqueAttributes.RemoveAtSwap(ExistingUniqueAttributeIndex);
					}
					else
					{
						// Create a unique attribute
						FUniqueAttribute& NewUniqueAttribute = UniqueAttributes.AddZeroed_GetRef();
						NewUniqueAttribute.Identifier = &AttributeIdentifier;
						NewUniqueAttribute.WeightIndex = WeightIndex;
						NewUniqueAttribute.DataPtr = ValuesArray[AttributeIndex].GetPtr<uint8>();
					}
				}
			}
		}
	}

	float FAttributeBlendData::GetContainerWeight(int32 ContainerIndex) const
	{
		// Check for float weights
		if (Weights.Num())
		{
			int32 WeightIndex = ContainerIndex;

			// Remap weight index if necessary
			if (WeightIndices.Num())
			{
				check(WeightIndices.IsValidIndex(ContainerIndex));
				WeightIndex = WeightIndices[ContainerIndex];
			}

			return Weights[WeightIndex];
		}

		return UniformWeight;
	}

	float FAttributeBlendData::GetBoneWeight(int32 ContainerIndex, int32 BoneIndex) const
	{
		// Check for FPerBoneBlendWeight data
		if (PerBoneWeights.Num())
		{
			ensure(PerBoneWeights.IsValidIndex(BoneIndex));

			// The ContainerIndex is offset by one when doing a filtered bone blend, as the Base Attributes take the 0 index
			if ((PerBoneWeights[BoneIndex].SourceIndex + 1) == ContainerIndex)
			{
				return PerBoneWeights[BoneIndex].BlendWeight;
			}
			// The base attributes weighting is the inverse of the filtered bone weight
			else if (ContainerIndex == 0)
			{
				return 1.0f - PerBoneWeights[BoneIndex].BlendWeight;
			}
		}

		// Check for float bone weights
		if (GetBoneWeights().Num())
		{
			ensure(Weights.IsValidIndex(BoneIndex));

			float BoneWeight = GetBoneWeights()[BoneIndex];
			// First attribute containers weighting is the inverse of the second container its weights
			BoneWeight = ContainerIndex == 0 ? 1.f - BoneWeight : BoneWeight;
			return BoneWeight;
		}

		// Check for FBlendSampleData data 
		if (BlendSampleDataCache.Num())
		{
			// Remap index if necessary
			const int32 SampleDataIndex = GetBlendSampleDataCacheIndices().Num() ? GetBlendSampleDataCacheIndices()[ContainerIndex] : ContainerIndex;
						
			const FBlendSampleData& BlendSampleData = BlendSampleDataCache[SampleDataIndex];
			const int32 PerBoneIndex = PerBoneInterpolationIndices[BoneIndex];

			// Blend-sample blending is only performed when they contain per-bone weights, if INDEX_NONE or out of range use the total weight instead
			if (PerBoneIndex != INDEX_NONE && BlendSampleData.PerBoneBlendData.IsValidIndex(PerBoneIndex))
			{
				return BlendSampleData.PerBoneBlendData[PerBoneIndex];
			}
			else
			{
				return BlendSampleData.GetClampedWeight();
			}
		}

		return 0.f;
	}

	bool FAttributeBlendData::HasBoneWeights() const
	{
		return HighestBoneWeightedIndices.Num() > 0;
	}
	bool FAttributeBlendData::HasContainerWeights() const
	{
		return HighestBoneWeightedIndices.Num() == 0;
	}
}}
