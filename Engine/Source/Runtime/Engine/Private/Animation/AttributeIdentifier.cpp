// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimData/AttributeIdentifier.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AttributeTypes.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttributeIdentifier)

#if WITH_EDITOR
FAnimationAttributeIdentifier UAnimationAttributeIdentifierExtensions::CreateAttributeIdentifier(UAnimationAsset* AnimationAsset, const FName AttributeName, const FName BoneName, UScriptStruct* AttributeType, bool bValidateExistsOnAsset /*= false*/)
{
	FAnimationAttributeIdentifier Identifier;

	// Ensure the type is valid and registered with the attributes system
	if (AttributeType)
	{
		if (UE::Anim::AttributeTypes::IsTypeRegistered(AttributeType))
		{
			if (AnimationAsset)
			{
				const USkeleton* Skeleton = AnimationAsset->GetSkeleton();
				if (Skeleton)
				{
					// Ensure that the request bone exists on the target asset its skeleton
					const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
					if (BoneIndex != INDEX_NONE)
					{
						Identifier.Name = AttributeName;
						Identifier.BoneName = BoneName;
						Identifier.BoneIndex = BoneIndex;
						Identifier.ScriptStruct = AttributeType;
						Identifier.ScriptStructPath = AttributeType;

						// If the user requested so, make sure the attribute exists on the asset, and if not reset the identifier to be invalid
						if (bValidateExistsOnAsset)
						{
							UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimationAsset);
							if (AnimSequence)
							{
								if (AnimSequence->GetDataModel()->FindAttribute(Identifier) == nullptr)
								{
									Identifier = FAnimationAttributeIdentifier();
									UE_LOGF(LogAnimation, Warning, "Attribute %ls does not exists on AnimationAsset %ls provided for CreateAttributeIdentifier", *Identifier.ToString(), *AnimationAsset->GetName());
								}
							}
							else
							{
								UE_LOGF(LogAnimation, Warning, "Cannot validate if Attribute %ls exists on AnimationAsset %ls because it is not an AnimationSequence", *Identifier.ToString(), *AnimationAsset->GetName());
								Identifier = FAnimationAttributeIdentifier();
							}
						}

						return Identifier;
					}
					else
					{
						UE_LOGF(LogAnimation, Warning, "Bone name %ls provided for CreateAttributeIdentifier does not exist on the Skeleton %ls", *BoneName.ToString(), *Skeleton->GetName());
					}
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Skeleton for provided Animation Asset %ls in CreateAttributeIdentifier is null", *AnimationAsset->GetName());
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Invalid Animation Asset provided for CreateAttributeIdentifier");
			}
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Attribute type %ls provided for CreateAttributeIdentifier has not been registered see AttributeTypes::RegisterType", *AttributeType->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "Invalid (null) Attribute type provided for CreateAttributeIdentifier");
	}

	return Identifier;
}
#endif // WITH_EDITOR
