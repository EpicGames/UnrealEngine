// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AttributeTypes.h"
#include "Animation/AttributeBlendOperator.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "AnimationRuntime.h"
#include "Misc/DelayedAutoRegister.h"
#include "Animation/AnimationSettings.h"
#include "StructUtils/UserDefinedStruct.h"

namespace UE
{
	namespace Anim
	{
		TArray<TWeakObjectPtr<const UScriptStruct>> AttributeTypes::RegisteredTypes;
		TArray<TUniquePtr<IAttributeBlendOperator>> AttributeTypes::Operators;
		TArray<TWeakObjectPtr<const UScriptStruct>> AttributeTypes::InterpolatableTypes;
		AttributeTypes::FOnAttributeTypesChanged AttributeTypes::OnAttributeTypesChangedDelegate;

		struct FAttributeTypeRegistrar
		{
			static void RegisterBuiltInTypes()
			{
				AttributeTypes::RegisterType<FFloatAnimationAttribute>();
				AttributeTypes::RegisterType<FIntegerAnimationAttribute>();
				AttributeTypes::RegisterType<FStringAnimationAttribute>();
				AttributeTypes::RegisterType<FTransformAnimationAttribute>();
				AttributeTypes::RegisterType<FVectorAnimationAttribute>();
				AttributeTypes::RegisterType<FQuaternionAnimationAttribute>();
			}

			static void RegisterUserDefinedStructTypes()
			{
				for (const TSoftObjectPtr<UUserDefinedStruct>& UserDefinedStruct : UAnimationSettings::Get()->UserDefinedStructAttributes)
				{
					AttributeTypes::RegisterNonBlendableType(UserDefinedStruct.LoadSynchronous());
				}
			}
		};

		static FDelayedAutoRegisterHelper DelayedBuiltInTypesInitializationHelper(EDelayedRegisterRunPhase::PreObjectSystemReady, []()
		{
			FAttributeTypeRegistrar::RegisterBuiltInTypes();
		});
		
		static FDelayedAutoRegisterHelper DelayedUserDefinedStructTypesInitializationHelper(EDelayedRegisterRunPhase::EndOfEngineInit, []()
		{
			FAttributeTypeRegistrar::RegisterUserDefinedStructTypes();
		});

		bool AttributeTypes::RegisterNonBlendableType(const UScriptStruct* InScriptStruct)
		{
			if (!InScriptStruct)
			{
				return false;
			}
				
			if (IsTypeRegistered(InScriptStruct))
			{
				return false;
			}
				
			AttributeTypes::RegisteredTypes.Add(InScriptStruct);
			AttributeTypes::Operators.Add(MakeUnique<FNonBlendableAttributeBlendOperator>(InScriptStruct));

			OnAttributeTypesChangedDelegate.Broadcast(InScriptStruct, true);
				
			return true;
		}

		void AttributeTypes::UnregisterType(const UScriptStruct* InScriptStruct)
		{
			if (UObjectInitialized())
			{
				const int32 Index = AttributeTypes::RegisteredTypes.IndexOfByKey(InScriptStruct);
				if (Index != INDEX_NONE)
				{
					AttributeTypes::RegisteredTypes.RemoveAtSwap(Index);
					AttributeTypes::Operators.RemoveAtSwap(Index);
				}
			}
				
			OnAttributeTypesChangedDelegate.Broadcast(InScriptStruct, false);
		}
	}
}

