// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AttributesContainer.h"

namespace UE::Anim
{
	const FName BoneAttributeNamespace{"bone"};
}
