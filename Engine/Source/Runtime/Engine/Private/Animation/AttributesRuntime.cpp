// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AttributesRuntime.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AnimationAsset.h"
#include "Animation/MirrorDataTable.h"
#include "Stats/Stats.h"

#include "Animation/AnimRootMotionProvider.h"
#include "Animation/IAttributeBlendOperator.h" 
#include "Animation/AttributeBlendData.h"
#include "Animation/AttributeTypes.h"
#include "AnimationRuntime.h"

namespace UE { namespace Anim {

#if WITH_EDITOR
void Attributes::GetAttributeValue(FStackAttributeContainer& OutAttributes, const FCompactPoseBoneIndex& PoseBoneIndex, const FAnimatedBoneAttribute& Attribute, const FAnimExtractContext& ExtractionContext)
{
	GetAttributeValue(OutAttributes, PoseBoneIndex, Attribute, ExtractionContext.CurrentTime);
}

void Attributes::GetAttributeValue(FStackAttributeContainer& OutAttributes, const FCompactPoseBoneIndex& PoseBoneIndex, const FAnimatedBoneAttribute& Attribute, double CurrentTime)
{
	// Evaluating a single attribute into a stack-based container
	if (Attribute.Identifier.IsValid())
	{
		uint8* AttributeDataPtr = OutAttributes.FindOrAdd(Attribute.Identifier.GetType(), FAttributeId(Attribute.Identifier.GetName(), PoseBoneIndex));
		check(AttributeDataPtr)
		Attribute.Curve.EvaluateToPtr(Attribute.Identifier.GetType(), CurrentTime, AttributeDataPtr);
	}
}
#endif // WITH_EDITOR

void Attributes::BlendAttributes(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, FStackAttributeContainer& OutAttributes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendAttributes);

	if (SourceAttributes.Num())
	{
		TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
		for (const FStackAttributeContainer& CustomAttributes : SourceAttributes)
		{
			UniqueStructs.Append(CustomAttributes.GetUniqueTypes());
		}
		
		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
		{
			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
			FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerContainerWeighted(SourceAttributes, SourceWeights, WeakScriptStruct.Get());
			Operator->Blend(AttributeBlendData, &OutAttributes);
		}
	}
}

void Attributes::BlendAttributes(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const TArrayView<const float> SourceWeights, FStackAttributeContainer& OutAttributes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendAttributes_Indirect);

	if (SourceAttributes.Num())
	{
		TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
		for (const FStackAttributeContainer* CustomAttributes : SourceAttributes)
		{
			UniqueStructs.Append(CustomAttributes->GetUniqueTypes());
		}

		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
		{
			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
			FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerContainerPtrWeighted(SourceAttributes, SourceWeights, WeakScriptStruct.Get());
			Operator->Blend(AttributeBlendData, &OutAttributes);
		}
	}
}

void Attributes::BlendAttributes(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, const TArrayView<const int32> SourceWeightsIndices, FStackAttributeContainer& OutAttributes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendAttributes_WeightsIndices);

	if (SourceAttributes.Num())
	{
		TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
		for (const FStackAttributeContainer& CustomAttributes : SourceAttributes)
		{
			UniqueStructs.Append(CustomAttributes.GetUniqueTypes());
		}

		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
		{
			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
			FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerContainerRemappedWeighted(SourceAttributes, SourceWeights, SourceWeightsIndices, WeakScriptStruct.Get());
			Operator->Blend(AttributeBlendData, &OutAttributes);
		}
	}
}

void Attributes::OverrideAttributes(const FStackAttributeContainer& SourceAttributes, FStackAttributeContainer& OutAttributes, float Weight)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_OverrideAttributes_Weighted);

	TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
	UniqueStructs.Append(SourceAttributes.GetUniqueTypes());
	UniqueStructs.Append(OutAttributes.GetUniqueTypes());

	const FStackAttributeContainer* Array[] = { &SourceAttributes };
	for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
	{
		const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);

		// Initialise blend data with uniform float weight
		FAttributeBlendData AttributeBlendData = FAttributeBlendData::SingleContainerUniformWeighted(Array, Weight, WeakScriptStruct.Get());
		Operator->Override(AttributeBlendData, &OutAttributes);
	}
}

void Attributes::AccumulateAttributes(const FStackAttributeContainer& SourceAttributes, FStackAttributeContainer& OutAttributes, float Weight, EAdditiveAnimationType AdditiveType)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AccumulateAttributes);

	TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
	UniqueStructs.Append(SourceAttributes.GetUniqueTypes());
	UniqueStructs.Append(OutAttributes.GetUniqueTypes());

	const FStackAttributeContainer* Array[] = { &SourceAttributes };
	for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
	{
		const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
		FAttributeBlendData AttributeBlendData = FAttributeBlendData::SingleAdditiveContainerUniformWeighted(Array, Weight, AdditiveType, WeakScriptStruct.Get());
		Operator->Accumulate(AttributeBlendData, &OutAttributes);
	}	
}
	
void Attributes::AccumulateAttributesWithOverrideWeights(const FStackAttributeContainer& SourceAttributes, FStackAttributeContainer& OutAttributes, const TMap<FAttributeId, float>& AttributeWeightOverrides, float Weight, EAdditiveAnimationType AdditiveType)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AccumulateAttributesWithOverrideWeights);

	TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
	UniqueStructs.Append(SourceAttributes.GetUniqueTypes());
	UniqueStructs.Append(OutAttributes.GetUniqueTypes());

	const FStackAttributeContainer* Array[] = { &SourceAttributes };
	for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
	{
		const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
		FAttributeBlendData AttributeBlendData = FAttributeBlendData::SingleAdditiveContainerWithOverrideWeights(Array, AttributeWeightOverrides, Weight, AdditiveType, WeakScriptStruct.Get());
		Operator->AccumulateWithOverrideWeights(AttributeBlendData, &OutAttributes);
	}	
}


void Attributes::ConvertToAdditive(const FStackAttributeContainer& BaseAttributes, FStackAttributeContainer& OutAdditiveAttributes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SubtractAttributes);

	TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
	UniqueStructs.Append(BaseAttributes.GetUniqueTypes());

	const FStackAttributeContainer* Array[] = { &BaseAttributes };
	for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
	{
		const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);

		// Subtract is simply an accumulate with -1.f blend weight
		FAttributeBlendData AttributeBlendData = FAttributeBlendData::SingleContainerUniformWeighted(Array, 1.f, WeakScriptStruct.Get());
		Operator->ConvertToAdditive(AttributeBlendData, &OutAdditiveAttributes);
	}
}

void Attributes::CopyAndRemapAttributes(const FMeshAttributeContainer& SourceAttributes, FStackAttributeContainer& TargetAttributes, const TMap<int32, int32>& BoneMapToSource, const FBoneContainer& RequiredBones)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CopyAndRemapAttributes);

	for (const TWeakObjectPtr<UScriptStruct> WeakScriptStruct : SourceAttributes.GetUniqueTypes())
	{
		UScriptStruct* ScriptStruct = WeakScriptStruct.Get();
		const int32 TypeIndex = SourceAttributes.FindTypeIndex(ScriptStruct);
		if (TypeIndex != INDEX_NONE)
		{
			const TArray<TWrappedAttribute<FDefaultAllocator>, FDefaultAllocator>& SourceValues = SourceAttributes.GetValues(TypeIndex);
			const TArray<FAttributeId, FDefaultAllocator>& AttributeIds = SourceAttributes.GetKeys(TypeIndex);

			// Try and remap all the source attributes to their respective new bone indices
			for (int32 EntryIndex = 0; EntryIndex < AttributeIds.Num(); ++EntryIndex)
			{
				const FAttributeId& AttributeId = AttributeIds[EntryIndex];

				const int32* Value = BoneMapToSource.Find(AttributeId.GetIndex());
				// If there is no remapping the attribute will be dropped
				if (Value)
				{
					const FMeshPoseBoneIndex MeshPoseIndex(*Value);
					const FSkeletonPoseBoneIndex RemappedPoseBoneIndex = RequiredBones.GetSkeletonPoseIndexFromMeshPoseIndex(MeshPoseIndex);
					const FCompactPoseBoneIndex CompactPoseIndex = RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(RemappedPoseBoneIndex);
					if (CompactPoseIndex.IsValid())
					{
						const FAttributeId NewInfo(AttributeId.GetName(), CompactPoseIndex.GetInt(), AttributeId.GetNamespace());
						uint8* NewAttribute = TargetAttributes.FindOrAdd(ScriptStruct, NewInfo);
						ScriptStruct->CopyScriptStruct(NewAttribute, SourceValues[EntryIndex].GetPtr<void>());
					}
				}
			}
		}
	}
}

void Attributes::InterpolateAttributes(FMeshAttributeContainer& FromAttributes, const FMeshAttributeContainer& ToAttributes, float Alpha)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_InterpolateAttributes);

	TSet<TWeakObjectPtr<UScriptStruct>> FromUniqueStructs;
	TSet<TWeakObjectPtr<UScriptStruct>> ToUniqueStructs;
	TSet<TWeakObjectPtr<UScriptStruct>> OverlappingStructs;
	{
		const TArray<TWeakObjectPtr<UScriptStruct>, FDefaultAllocator>& SourceUniqueTypes = FromAttributes.GetUniqueTypes();
		const TArray<TWeakObjectPtr<UScriptStruct>, FDefaultAllocator>& OutUniqueTypes = ToAttributes.GetUniqueTypes();

		for (const auto& Type : SourceUniqueTypes)
		{
			if (OutUniqueTypes.Contains(Type))
			{
				OverlappingStructs.Add(Type);
			}
			else
			{
				FromUniqueStructs.Add(Type);
			}
		}

		for (const auto& Type : OutUniqueTypes)
		{
			if (!SourceUniqueTypes.Contains(Type))
			{
				ToUniqueStructs.Add(Type);
			}
		}
	}

	// Handle unique types first
	{
		// From
		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : FromUniqueStructs)
		{
			const int32 TypeIndex = FromAttributes.FindTypeIndex(WeakScriptStruct.Get());
			const TArray<FAttributeId, FDefaultAllocator>& Identifiers = FromAttributes.GetKeys(TypeIndex);
			const TArray<TWrappedAttribute<FDefaultAllocator>, FDefaultAllocator>& Values = FromAttributes.GetValues(TypeIndex);

			FWrappedAttribute DefaultValue(WeakScriptStruct.Get());
			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
			for (int32 EntryIndex = 0; EntryIndex < Identifiers.Num(); ++EntryIndex)
			{
				uint8* OutAttributeData = FromAttributes.Find(WeakScriptStruct.Get(), Identifiers[EntryIndex]);
				ensure(OutAttributeData);

				// Interpolate, from default value to stored value
				Operator->Interpolate(Values[EntryIndex].GetPtr<void>(), DefaultValue.GetPtr<void>(), 1.f - Alpha, (void*)OutAttributeData);
			}
		}

		// To 
		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : ToUniqueStructs)
		{
			const int32 TypeIndex = ToAttributes.FindTypeIndex(WeakScriptStruct.Get());
			const TArray<FAttributeId, FDefaultAllocator>& Identifiers = ToAttributes.GetKeys(TypeIndex);
			const TArray<TWrappedAttribute<FDefaultAllocator>, FDefaultAllocator>& Values = ToAttributes.GetValues(TypeIndex);

			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
			for (int32 EntryIndex = 0; EntryIndex < Identifiers.Num(); ++EntryIndex)
			{
				const uint8* AttributeData = ToAttributes.Find(WeakScriptStruct.Get(), Identifiers[EntryIndex]);
				ensure(AttributeData);

				uint8* OutAttributeData = FromAttributes.Add(WeakScriptStruct.Get(), Identifiers[EntryIndex]);
				ensure(OutAttributeData);

				Operator->Interpolate(OutAttributeData, AttributeData, Alpha, OutAttributeData);
			}
		}
	}

	// Overlapping types	
	for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : OverlappingStructs)
	{
		const int32 FromTypeIndex = FromAttributes.FindTypeIndex(WeakScriptStruct.Get());
		const TArray<FAttributeId, FDefaultAllocator>& FromIdentifiers = FromAttributes.GetKeys(FromTypeIndex);
		const TArray<TWrappedAttribute<FDefaultAllocator>, FDefaultAllocator>& FromValues = FromAttributes.GetValues(FromTypeIndex);

		const int32 ToTypeIndex = ToAttributes.FindTypeIndex(WeakScriptStruct.Get());
		const TArray<FAttributeId, FDefaultAllocator>& ToIdentifiers = ToAttributes.GetKeys(ToTypeIndex);
		const TArray<TWrappedAttribute<FDefaultAllocator>, FDefaultAllocator>& ToValues = ToAttributes.GetValues(ToTypeIndex);

		const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);

		FWrappedAttribute DefaultValue(WeakScriptStruct.Get());

		for (int32 EntryIndex = 0; EntryIndex < FromIdentifiers.Num(); ++EntryIndex)
		{
			const int32 FromValueIndex = EntryIndex;
			const int32 ToValueIndex = ToIdentifiers.IndexOfByKey(FromIdentifiers[EntryIndex]);

			uint8* FromAttributeData = FromAttributes.Find(WeakScriptStruct.Get(), FromIdentifiers[FromValueIndex]);
			ensure(FromAttributeData);

			const uint8* ToAttributeData = nullptr;
			if (ToValueIndex != INDEX_NONE)
			{
				// Exists in both
				ToAttributeData = ToAttributes.Find(WeakScriptStruct.Get(), ToIdentifiers[ToValueIndex]);
				ensure(ToAttributeData);
			}
			else
			{
				// Exists in From only
				ToAttributeData = DefaultValue.GetPtr<uint8>();
			}

			ensure(ToAttributeData);

			Operator->Interpolate(FromAttributeData, ToAttributeData, Alpha, FromAttributeData);
		}

		for (int32 EntryIndex = 0; EntryIndex < ToIdentifiers.Num(); ++EntryIndex)
		{
			const int32 ToValueIndex = EntryIndex;
			const int32 FromValueIndex = FromIdentifiers.IndexOfByKey(ToIdentifiers[EntryIndex]);
			if (FromValueIndex == INDEX_NONE)
			{
				// Exists only in to
				const uint8* ToAttributeData = ToAttributes.Find(WeakScriptStruct.Get(), ToIdentifiers[ToValueIndex]);
				ensure(ToAttributeData);

				uint8* FromAttributeData = FromAttributes.Add(WeakScriptStruct.Get(), ToIdentifiers[ToValueIndex]);
				ensure(FromAttributeData);

				Operator->Interpolate(DefaultValue.GetPtr<uint8>(), ToAttributeData, Alpha, FromAttributeData);
			}
		}
	}
}

void Attributes::BlendAttributesPerBone(const FStackAttributeContainer& SourceAttributes1, const FStackAttributeContainer& SourceAttributes2, const TArrayView<const float> WeightsOfSource2, FStackAttributeContainer& OutAttributes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendAttributesPerBone);

	TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
	UniqueStructs.Append(SourceAttributes1.GetUniqueTypes());
	UniqueStructs.Append(SourceAttributes2.GetUniqueTypes());

	for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
	{
		const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
		FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerBoneSingleContainerWeighted(SourceAttributes1, SourceAttributes2, WeightsOfSource2, WeakScriptStruct.Get());
		Operator->BlendPerBone(AttributeBlendData, &OutAttributes);
	}
}

void Attributes::BlendAttributesPerBone(TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> PerBoneInterpolationIndices, TArrayView<const FBlendSampleData> BlendSampleDataCache, FStackAttributeContainer& OutAttributes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendAttributesPerBoneBlendSample);

	TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
	for (const FStackAttributeContainer& CustomAttributes : SourceAttributes)
	{
		UniqueStructs.Append(CustomAttributes.GetUniqueTypes());
	}

	// In case there are no per-bone weights the attributes can be blended with container-level weights
	const bool bContainsPerBoneWeights = BlendSampleDataCache[0].PerBoneBlendData.Num() > 0;
	if (bContainsPerBoneWeights)
	{
		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
		{
			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);

			FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerBoneBlendSamples(SourceAttributes, PerBoneInterpolationIndices, BlendSampleDataCache, WeakScriptStruct.Get());
			Operator->BlendPerBone(AttributeBlendData, &OutAttributes);
		}
	}
	else
	{
		TArrayView<float> Weights((float*)FMemory_Alloca(BlendSampleDataCache.Num() * sizeof(float)), BlendSampleDataCache.Num());
		for (int32 WeightIndex = 0; WeightIndex < BlendSampleDataCache.Num(); ++WeightIndex)
		{
			Weights[WeightIndex] = BlendSampleDataCache[WeightIndex].GetClampedWeight();
		}

		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
		{
			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);

			FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerContainerWeighted(SourceAttributes, Weights, WeakScriptStruct.Get());
			Operator->Blend(AttributeBlendData, &OutAttributes);
		}
	}
}

void Attributes::BlendAttributesPerBone(TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> PerBoneInterpolationIndices, TArrayView<const FBlendSampleData> BlendSampleDataCache, TArrayView<const int32> BlendSampleDataCacheIndices, FStackAttributeContainer& OutAttributes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendAttributesPerBoneBlendSample);

	TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
	for (const FStackAttributeContainer& CustomAttributes : SourceAttributes)
	{
		UniqueStructs.Append(CustomAttributes.GetUniqueTypes());
	}

	// In case there are no per-bone weights the attributes can be blended with container-level weights
	const bool bContainsPerBoneWeights = BlendSampleDataCache[0].PerBoneBlendData.Num() > 0;
	if (bContainsPerBoneWeights)
	{
		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
		{
			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);

			FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerBoneRemappedBlendSamples(SourceAttributes, PerBoneInterpolationIndices, BlendSampleDataCache, BlendSampleDataCacheIndices, WeakScriptStruct.Get());
			Operator->BlendPerBone(AttributeBlendData, &OutAttributes);
		}
	}
	else
	{
		TArrayView<float> Weights((float*)FMemory_Alloca(BlendSampleDataCacheIndices.Num() * sizeof(float)), BlendSampleDataCacheIndices.Num());
		for (int32 WeightIndex = 0; WeightIndex < BlendSampleDataCacheIndices.Num(); ++WeightIndex)
		{
			Weights[WeightIndex] = BlendSampleDataCache[BlendSampleDataCacheIndices[WeightIndex]].GetClampedWeight();
		}

		for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
		{
			const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);

			FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerContainerWeighted(SourceAttributes, Weights, WeakScriptStruct.Get());
			Operator->Blend(AttributeBlendData, &OutAttributes);
		}
	}
}

void Attributes::BlendAttributesPerBoneFilter(const FStackAttributeContainer& BaseAttributes, const TArrayView<const FStackAttributeContainer> BlendAttributes, const TConstArrayView<FPerBoneBlendWeight> BoneBlendWeights, FStackAttributeContainer& OutAttributes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendAttributesPerBoneFilter);

	TSet<TWeakObjectPtr<UScriptStruct>, DefaultKeyFuncs<TWeakObjectPtr<UScriptStruct>>, TInlineSetAllocator<4>> UniqueStructs;
	for (const FStackAttributeContainer& CustomAttributes : BlendAttributes)
	{
		UniqueStructs.Append(CustomAttributes.GetUniqueTypes());
	}

	UniqueStructs.Append(BaseAttributes.GetUniqueTypes());

	for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : UniqueStructs)
	{
		const IAttributeBlendOperator* Operator = AttributeTypes::GetTypeOperator(WeakScriptStruct);
		FAttributeBlendData AttributeBlendData = FAttributeBlendData::PerBoneFilteredWeighted(BaseAttributes, BlendAttributes, BoneBlendWeights, WeakScriptStruct.Get());
		Operator->BlendPerBone(AttributeBlendData, &OutAttributes);
	}
}

void Attributes::MirrorAttributes(FStackAttributeContainer& CustomAttributes, const UMirrorDataTable& MirrorTable)
{
	static const TArray<FCompactPoseBoneIndex> CompactPoseMirrorBones;
	Attributes::MirrorAttributes(CustomAttributes, MirrorTable, CompactPoseMirrorBones);
}



void Attributes::MirrorAttributes(FStackAttributeContainer& CustomAttributes, const UMirrorDataTable& MirrorTable, const TArray<FCompactPoseBoneIndex>& CompactPoseMirrorBones)
{
	struct FMirrorSet
	{
		/** Pointers to attribute values */
		TArray<uint8*, FAnimStackAllocator> DataPtrsA;
		TArray<uint8*, FAnimStackAllocator> DataPtrsB;
		/** Identifier of the attribute */
		TArray <const FAttributeId*, FAnimStackAllocator> IdentifiersA;
		TArray <const FAttributeId*, FAnimStackAllocator> IdentifiersB;
		int32 MirrorIndexA;
		int32 MirrorIndexB;
		void Reset() { DataPtrsA.Reset(); DataPtrsB.Reset(), IdentifiersA.Reset(); IdentifiersB.Reset(); }
	};

	TArray<FMirrorSet, FAnimStackAllocator> MirrorSets;

	auto AddUnique = [&MirrorSets](int32 IndexA, int32 IndexB) -> bool
	{
		for (FMirrorSet& CurSet : MirrorSets)
		{
			if ( (CurSet.MirrorIndexA == IndexA && CurSet.MirrorIndexB == IndexB) ||
				 (CurSet.MirrorIndexB == IndexA && CurSet.MirrorIndexA == IndexB) )
			{
				return false;
			}
		}
		FMirrorSet& NewSet = MirrorSets.Emplace_GetRef();
		NewSet.MirrorIndexA = IndexA;
		NewSet.MirrorIndexB = IndexB;
		return true;
	};

	// @todo: initialize this array with a properly sized TInlineAllocator
	TArray<int32> SortedBoneIndices;
	for (const TWeakObjectPtr<UScriptStruct>& WeakScriptStruct : CustomAttributes.GetUniqueTypes())
	{
		const int32 TypeIndex = CustomAttributes.FindTypeIndex(WeakScriptStruct.Get());
		if (TypeIndex != INDEX_NONE)
		{
			MirrorSets.Reset();
			const TConstArrayView<int32> UniqueBoneIndices = CustomAttributes.GetUniqueTypedBoneIndices(TypeIndex);
			SortedBoneIndices = UniqueBoneIndices; 
			SortedBoneIndices.Sort();

			if(CompactPoseMirrorBones.Num())
			{
			    for (int32 CurBoneIndex : UniqueBoneIndices)
			    {
				    // only handle simple swaps - both entries exist and are different
				    if (CompactPoseMirrorBones.IsValidIndex(CurBoneIndex))
				    {
					    int32 MirrorBoneIndex = CompactPoseMirrorBones[CurBoneIndex].GetInt();
					    if (CurBoneIndex != MirrorBoneIndex && CompactPoseMirrorBones.IsValidIndex(MirrorBoneIndex))
					    {
						    int32 SwapCheckBoneIndex = CompactPoseMirrorBones[MirrorBoneIndex].GetInt();
						    if (SwapCheckBoneIndex == CurBoneIndex && Algo::BinarySearch(SortedBoneIndices, MirrorBoneIndex))
						    {
							    AddUnique(CurBoneIndex, MirrorBoneIndex);
						    }
					    }
				    }				
			    }
			}
            else
            {
				// Deprecated behaviour support, remove when deleting other signature
			    for (int32 CurBoneIndex : UniqueBoneIndices)
			    {
				    // Mirror tables are stored using skeleton indices, but are assumed to be used interchangeably with compact pose indices here.
				    int32 MirrorBoneIndex = MirrorTable.BoneToMirrorBoneIndex[CurBoneIndex].GetInt();
				    int32 SwapCheckBoneIndex = MirrorTable.BoneToMirrorBoneIndex[MirrorBoneIndex].GetInt();
    
				    // only handle simple swaps - both entries exist and are different
				    if (MirrorBoneIndex != INDEX_NONE && CurBoneIndex != INDEX_NONE && CurBoneIndex != MirrorBoneIndex &&
					    SwapCheckBoneIndex == CurBoneIndex && Algo::BinarySearch(SortedBoneIndices, MirrorBoneIndex))
				    {
					    AddUnique(CurBoneIndex, MirrorBoneIndex);
				    }
			    }
			}

			TArrayView<TWrappedAttribute<FAnimStackAllocator>> ValuesArray = FStackAttributeContainerAccessor::GetValues(CustomAttributes, TypeIndex);
			const TConstArrayView<FAttributeId> AttributeIdentifiers = CustomAttributes.GetKeys(TypeIndex);

			// gather attributes that are on mirrored bones 
			for (int32 AttributeIndex = 0; AttributeIndex < AttributeIdentifiers.Num(); ++AttributeIndex)
			{
				const FAttributeId& AttributeIdentifier = AttributeIdentifiers[AttributeIndex];
				uint8* DataPtr = ValuesArray[AttributeIndex].GetPtr<uint8>();
				for (FMirrorSet& CurSet : MirrorSets)
				{
					if (CurSet.MirrorIndexA == AttributeIdentifier.GetIndex())
					{
						CurSet.IdentifiersA.Emplace(&AttributeIdentifier);
						CurSet.DataPtrsA.Emplace(DataPtr);
					}
					else if (CurSet.MirrorIndexB == AttributeIdentifier.GetIndex())
					{
						CurSet.IdentifiersB.Emplace(&AttributeIdentifier);
						CurSet.DataPtrsB.Emplace(DataPtr);
					}
				}
			}

			for (FMirrorSet& CurSet : MirrorSets)
			{
				for (int32 IndexA = 0; IndexA < CurSet.IdentifiersA.Num(); ++IndexA)
				{
					const FAttributeId* IndexAId = CurSet.IdentifiersA[IndexA];
					uint8* IndexADataPtr = CurSet.DataPtrsA[IndexA];
					for (int32 IndexB = 0; IndexB < CurSet.IdentifiersB.Num(); ++IndexB)
					{
						const FAttributeId* IndexBId = CurSet.IdentifiersB[IndexB];
						if (IndexAId->GetName() == IndexBId->GetName() && IndexAId->GetNamespace() == IndexBId->GetNamespace())
						{
							uint8* IndexBDataPtr = CurSet.DataPtrsB[IndexB];
							TWrappedAttribute<FAnimStackAllocator> SwapStruct(WeakScriptStruct.Get());
							WeakScriptStruct->CopyScriptStruct(SwapStruct.GetPtr<void>(), IndexADataPtr);
							WeakScriptStruct->CopyScriptStruct(IndexADataPtr, IndexBDataPtr);
							WeakScriptStruct->CopyScriptStruct(IndexBDataPtr, SwapStruct.GetPtr<void>());
							break;
						}
					}
				}
			}
		} 
	}

	const IAnimRootMotionProvider* RootMotionProvider = IAnimRootMotionProvider::Get();
	
	if (RootMotionProvider != nullptr && MirrorTable.bMirrorRootMotion)
	{
		FTransform RootMotionTransformDelta = FTransform::Identity; 
		if (RootMotionProvider->ExtractRootMotion(CustomAttributes, RootMotionTransformDelta)) 
		{
			auto MirrorTransform = [](const FTransform& SourceTransform, EAxis::Type MirrorAxis) 
			{
				FVector T = SourceTransform.GetTranslation(); 
				T = FAnimationRuntime::MirrorVector(T, MirrorAxis); 
				FQuat Q = SourceTransform.GetRotation();
				Q = FAnimationRuntime::MirrorQuat(Q, MirrorAxis); 

				FVector S = SourceTransform.GetScale3D(); 

				return FTransform(Q, T, S); 
			};

			RootMotionTransformDelta = MirrorTransform(RootMotionTransformDelta, MirrorTable.MirrorAxis); 
			RootMotionProvider->OverrideRootMotion(RootMotionTransformDelta, CustomAttributes);
		}
	}
}

ECustomAttributeBlendType Attributes::GetAttributeBlendType(const FName& InName)
{
	const UAnimationSettings* Settings = UAnimationSettings::Get();
	const ECustomAttributeBlendType* ModePtr = Settings->AttributeBlendModes.Find(InName);
	return ModePtr ? *ModePtr : Settings->DefaultAttributeBlendMode;
}

ECustomAttributeBlendType Attributes::GetAttributeBlendType(const FAttributeId& Info)
{
	const UAnimationSettings* Settings = UAnimationSettings::Get();

	TArray<FName> Names;
	Settings->AttributeBlendModes.GenerateKeyArray(Names);
	const FName* NamePtr = Names.FindByPredicate([Info](const FName& Name)
	{
		return Name == Info.GetName();
	});

	return NamePtr ? Settings->AttributeBlendModes.FindChecked(*NamePtr) : Settings->DefaultAttributeBlendMode;
}

}}
