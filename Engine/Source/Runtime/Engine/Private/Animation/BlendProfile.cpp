// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/BlendProfile.h"
#include "AlphaBlend.h"
#include "Animation/AnimationAsset.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendProfile)

void IBlendProfileInterface::UpdateBoneWeights(FBlendSampleData& InOutCurrentData, const FAlphaBlend& BlendInfo, float BlendStartAlpha, float MainWeight, bool bInverse) const
{
	const EBlendProfileMode Mode = GetMode();

	for (int32 PerBoneIndex = 0; PerBoneIndex < InOutCurrentData.PerBoneBlendData.Num(); ++PerBoneIndex)
	{
		InOutCurrentData.PerBoneBlendData[PerBoneIndex] =  UBlendProfile::CalculateBoneWeight(GetBoneBlendScale(PerBoneIndex), Mode, BlendInfo, BlendStartAlpha, MainWeight, bInverse);
	}
}

void FBlendProfileInterfaceWrapper::UpdateCachedBlendProfile()
{
	if (!bIsSkeletonBlendProfile && BlendProfileProvider.GetObject() && BlendProfile)
	{
		if (BlendProfileProvider.GetObject() && BlendProfile)
		{
			BlendProfileProvider->ConstructBlendProfile(BlendProfile);
		}
		else
		{
			BlendProfile = nullptr;
		}
	}
}

void FBlendProfileInterfaceWrapper::SetBlendProfileProvider(TObjectPtr<UObject> Provider, IBlendProfileProviderInterface* ProviderInterface, UObject* Outer)
{
	BlendProfileProvider.SetObject(Provider);
	BlendProfileProvider.SetInterface(ProviderInterface);

	if (BlendProfileProvider)
	{
		if (!BlendProfile || BlendProfile->GetOuter() != Outer)
		{
			BlendProfile = NewObject<UBlendProfile>(Outer);
		}

		BlendProfile->ClearEntries();
		BlendProfileProvider->ConstructBlendProfile(BlendProfile);
	}
	
	bIsSkeletonBlendProfile = false;
}

void FBlendProfileInterfaceWrapper::SetSkeletonBlendProfile(TObjectPtr<UBlendProfile> InBlendProfile)
{
	BlendProfileProvider.SetObject(nullptr);
	BlendProfileProvider.SetInterface(nullptr);

	BlendProfile = InBlendProfile;
	
	bIsSkeletonBlendProfile = true;
}

UBlendProfile::UBlendProfile()
	: OwningSkeleton(nullptr)
	, Mode(EBlendProfileMode::WeightFactor)
{
	// Set up our owning skeleton and initialise bone references
	if(USkeleton* OuterAsSkeleton = Cast<USkeleton>(GetOuter()))
	{
		SetSkeleton(OuterAsSkeleton);
	}
}

void UBlendProfile::SetBoneBlendScale(int32 InBoneIdx, float InScale, bool bRecurse, bool bCreate)
{
	// Set the requested bone, then children if necessary
	SetSingleBoneBlendScale(InBoneIdx, InScale, bCreate);

	if(bRecurse)
	{
		const FReferenceSkeleton& RefSkeleton = OwningSkeleton->GetReferenceSkeleton();
		const int32 NumBones = RefSkeleton.GetNum();
		for(int32 ChildIdx = InBoneIdx + 1 ; ChildIdx < NumBones ; ++ChildIdx)
		{
			if(RefSkeleton.BoneIsChildOf(ChildIdx, InBoneIdx))
			{
				SetSingleBoneBlendScale(ChildIdx, InScale, bCreate);
			}
		}
	}
}

void UBlendProfile::SetBoneBlendScale(const FName& InBoneName, float InScale, bool bRecurse, bool bCreate)
{
	int32 BoneIndex = OwningSkeleton->GetReferenceSkeleton().FindBoneIndex(InBoneName);

	SetBoneBlendScale(BoneIndex, InScale, bRecurse, bCreate);
}

void UBlendProfile::RemoveEntry(int32 InBoneIdx)
{
	Modify();
	ProfileEntries.RemoveAll([InBoneIdx](const FBlendProfileBoneEntry& Current)
		{
			return Current.BoneReference.BoneIndex == InBoneIdx;
		});
}

void UBlendProfile::RefreshBoneEntry(int32 InBoneIndex)
{
	FBlendProfileBoneEntry* FoundEntry = ProfileEntries.FindByPredicate([InBoneIndex](const FBlendProfileBoneEntry& Entry)
		{
			return Entry.BoneReference.BoneIndex == InBoneIndex;
		});

	if (FoundEntry)
	{
		FoundEntry->BoneReference.BoneName = OwningSkeleton->GetReferenceSkeleton().GetBoneName(InBoneIndex);
		FoundEntry->BoneReference.Initialize(OwningSkeleton);
	}
}

void UBlendProfile::RefreshBoneEntriesFromName()
{
	for (FBlendProfileBoneEntry& Entry : ProfileEntries)
	{
		Entry.BoneReference.Initialize(OwningSkeleton);
	}
}

void UBlendProfile::CleanupBoneEntries()
{
	ProfileEntries.RemoveAll([](const FBlendProfileBoneEntry& Entry)
		{
			return !Entry.BoneReference.HasValidSetup();
		});
}

const FBlendProfileBoneEntry& UBlendProfile::GetEntry(const int32 InEntryIdx) const
{
	return ProfileEntries[InEntryIdx];
}

float UBlendProfile::GetBoneBlendScale(int32 InBoneIdx) const
{
	const FBlendProfileBoneEntry* FoundEntry = ProfileEntries.FindByPredicate([InBoneIdx](const FBlendProfileBoneEntry& Entry)
	{
		return Entry.BoneReference.BoneIndex == InBoneIdx;
	});

	if(FoundEntry)
	{
		return FoundEntry->BlendScale;
	}

	return GetDefaultBlendScale();
}

float UBlendProfile::GetBoneBlendScale(const FName& InBoneName) const
{
	const FBlendProfileBoneEntry* FoundEntry = ProfileEntries.FindByPredicate([InBoneName](const FBlendProfileBoneEntry& Entry)
	{
		return Entry.BoneReference.BoneName == InBoneName;
	});

	if(FoundEntry)
	{
		return FoundEntry->BlendScale;
	}

	return GetDefaultBlendScale();
}

void UBlendProfile::SetSkeleton(USkeleton* InSkeleton)
{
	OwningSkeleton = InSkeleton;

	if(OwningSkeleton)
	{
		// Initialise Current profile entries
		for(FBlendProfileBoneEntry& Entry : ProfileEntries)
		{
			Entry.BoneReference.Initialize(OwningSkeleton);
		}
	}

	// Remove any entries for bones that aren't mapped
	ProfileEntries.RemoveAll([](const FBlendProfileBoneEntry& Current)
		{
			return Current.BoneReference.BoneIndex == INDEX_NONE;
		});
}

void UBlendProfile::PostLoad()
{
	Super::PostLoad();

	if(OwningSkeleton)
	{
		// Initialise Current profile entries
		for(FBlendProfileBoneEntry& Entry : ProfileEntries)
		{
			Entry.BoneReference.Initialize(OwningSkeleton);
		}
	}

#if WITH_EDITOR
	// Remove any entries for bones that aren't mapped
	ProfileEntries.RemoveAll([](const FBlendProfileBoneEntry& Current)
		{
			return Current.BoneReference.BoneIndex == INDEX_NONE;
		});
#endif
}

int32 UBlendProfile::GetEntryIndex(const FSkeletonPoseBoneIndex InBoneIdx) const
{
	for(int32 Idx = 0 ; Idx < ProfileEntries.Num() ; ++Idx)
	{
		const FBlendProfileBoneEntry& Entry = ProfileEntries[Idx];
		if(Entry.BoneReference.BoneIndex == InBoneIdx.GetInt())
		{
			return Idx;
		}
	}
	return INDEX_NONE;
}

int32 UBlendProfile::GetEntryIndex(const FName& InBoneName) const
{
	for(int32 Idx = 0 ; Idx < ProfileEntries.Num() ; ++Idx)
	{
		const FBlendProfileBoneEntry& Entry = ProfileEntries[Idx];
		if(Entry.BoneReference.BoneName == InBoneName)
		{
			return Idx;
		}
	}
	return INDEX_NONE;
}

float UBlendProfile::GetEntryBlendScale(const int32 InEntryIdx) const
{
	if(ProfileEntries.IsValidIndex(InEntryIdx))
	{
		return ProfileEntries[InEntryIdx].BlendScale;
	}
	// No overridden blend scale, return no scale
	return GetDefaultBlendScale();
}

int32 UBlendProfile::GetPerBoneInterpolationIndex(const FCompactPoseBoneIndex& InCompactPoseBoneIndex, const FBoneContainer& BoneContainer, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const
{
	return GetEntryIndex(BoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(InCompactPoseBoneIndex));
}

int32 UBlendProfile::GetPerBoneInterpolationIndex(const FSkeletonPoseBoneIndex InSkeletonBoneIndex, const USkeleton* TargetSkeleton, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const
{
	return GetEntryIndex(InSkeletonBoneIndex);
}

void UBlendProfile::SetSingleBoneBlendScale(int32 InBoneIdx, float InScale, bool bCreate /*= false*/)
{
	FBlendProfileBoneEntry* Entry = ProfileEntries.FindByPredicate([InBoneIdx](const FBlendProfileBoneEntry& InEntry)
	{
		return InEntry.BoneReference.BoneIndex == InBoneIdx;
	});

	if(!Entry && bCreate)
	{
		Entry = &ProfileEntries[ProfileEntries.AddZeroed()];
		Entry->BoneReference.BoneName = OwningSkeleton->GetReferenceSkeleton().GetBoneName(InBoneIdx);
		Entry->BoneReference.Initialize(OwningSkeleton);
		Entry->BlendScale = InScale;
	}

	if(Entry)
	{
		Entry->BlendScale = InScale;

		// Remove any entry that gets set back to DefautBlendScale - so we only store entries that actually contain a scale
		if(Entry->BlendScale == GetDefaultBlendScale())
		{
			ProfileEntries.RemoveAll([InBoneIdx](const FBlendProfileBoneEntry& Current)
			{
				return Current.BoneReference.BoneIndex == InBoneIdx;
			});
		}
	}
}

void UBlendProfile::FillBoneScalesArray(TArray<float>& OutBoneBlendProfileFactors, const FBoneContainer& BoneContainer) const
{
	const int32 NumBones = BoneContainer.GetCompactPoseNumBones();
	OutBoneBlendProfileFactors.Reset(NumBones);
	OutBoneBlendProfileFactors.AddUninitialized(NumBones);

	// Fill the bone values with defaults values.
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		OutBoneBlendProfileFactors[Index] = 1.0f;
	}

	// Overwrite the values of the bones that are inside the blend profile.
	// Since the bones in the blend profile are stored as skeleton indices we need to remap them into our compact pose.
	const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(OwningSkeleton, BoneContainer.GetSkeletonAsset());
	if (SkeletonRemapping.IsValid())
	{
		for (int32 Index = 0; Index < ProfileEntries.Num(); Index++)
		{
			const int32 SkeletonBoneIndex = ProfileEntries[Index].BoneReference.BoneIndex;
			const int32 TargetBoneIndex = SkeletonRemapping.GetTargetSkeletonBoneIndex(SkeletonBoneIndex);
			const FCompactPoseBoneIndex PoseBoneIndex = BoneContainer.GetCompactPoseIndexFromSkeletonIndex(TargetBoneIndex);
			if (PoseBoneIndex.IsValid())
			{
				OutBoneBlendProfileFactors[PoseBoneIndex.GetInt()] = GetEntryBlendScale(Index);
			}
		}
	}
	else // We don't use skeleton remapping, slightly more optimized.
	{
		for (int32 Index = 0; Index < ProfileEntries.Num(); Index++)
		{
			const int32 SkeletonBoneIndex = ProfileEntries[Index].BoneReference.BoneIndex;
			const FCompactPoseBoneIndex PoseBoneIndex = BoneContainer.GetCompactPoseIndexFromSkeletonIndex(SkeletonBoneIndex);
			if (PoseBoneIndex.IsValid())
			{
				OutBoneBlendProfileFactors[PoseBoneIndex.GetInt()] = GetEntryBlendScale(Index);
			}
		}
	}
}

void UBlendProfile::FillSkeletonBoneDurationsArray(TCustomBoneIndexArrayView<float, FSkeletonPoseBoneIndex> OutDurationPerBone, float Duration, const USkeleton* TargetSkeleton) const
{
	check(OwningSkeleton != nullptr);
	if (TargetSkeleton == nullptr)
	{
		TargetSkeleton = OwningSkeleton;
	}

	const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(OwningSkeleton, TargetSkeleton);

	for(float& BoneDuration: OutDurationPerBone)
	{
		BoneDuration = Duration;
	}

	switch (Mode)
	{
		case EBlendProfileMode::TimeFactor:
		{
			if (SkeletonRemapping.IsValid())
			{
				for (const FBlendProfileBoneEntry& Entry : ProfileEntries)
				{
					const FSkeletonPoseBoneIndex SkeletonBoneIndex(SkeletonRemapping.GetTargetSkeletonBoneIndex(Entry.BoneReference.BoneIndex));
					if (SkeletonBoneIndex != INDEX_NONE)
					{
						OutDurationPerBone[SkeletonBoneIndex] *= Entry.BlendScale;
					}
				}
			}
			else
			{
				for (const FBlendProfileBoneEntry& Entry : ProfileEntries)
				{
					const FSkeletonPoseBoneIndex SkeletonBoneIndex(Entry.BoneReference.BoneIndex);
					if (SkeletonBoneIndex != INDEX_NONE)
					{
						OutDurationPerBone[SkeletonBoneIndex] *= Entry.BlendScale;
					}
				}
			}
		}
		break;

		case EBlendProfileMode::WeightFactor:
		{
			if (SkeletonRemapping.IsValid())
			{
				for (const FBlendProfileBoneEntry& Entry : ProfileEntries)
				{
					const FSkeletonPoseBoneIndex SkeletonBoneIndex(SkeletonRemapping.GetTargetSkeletonBoneIndex(Entry.BoneReference.BoneIndex));
					if (Entry.BlendScale > UE_SMALL_NUMBER && SkeletonBoneIndex != INDEX_NONE)
					{
						OutDurationPerBone[SkeletonBoneIndex] /= Entry.BlendScale;
					}
				}
			}
			else
			{
				for (const FBlendProfileBoneEntry& Entry : ProfileEntries)
				{
					const FSkeletonPoseBoneIndex SkeletonBoneIndex(Entry.BoneReference.BoneIndex);
					if (Entry.BlendScale > UE_SMALL_NUMBER && SkeletonBoneIndex != INDEX_NONE)
					{
						OutDurationPerBone[SkeletonBoneIndex] /= Entry.BlendScale;
					}
				}
			}
		}
		break;

		default:
		{
			checkf(false, TEXT("The selected Blend Profile Mode is not supported (Mode=%d)"), EnumToUnderlyingType(Mode));
		}
		break;
	}
}

float UBlendProfile::CalculateBoneWeight(float BoneFactor, EBlendProfileMode Mode, const FAlphaBlend& BlendInfo, float BlendStartAlpha, float MainWeight, bool bInverse)
{
	switch (Mode)
	{
		// The per bone value is a factor of the transition time, where 0.5 means half the transition time, 0.1 means one tenth of the transition time, etc.
		case EBlendProfileMode::TimeFactor:
		{
			// Most bones will have a bone factor of 1, so let's optimize that case.
			// Basically it means it will just follow the main weight.
			if (BoneFactor >= 1.0f - ZERO_ANIMWEIGHT_THRESH)
			{
				return !bInverse ? MainWeight : 1.0f - MainWeight;
			}

			// Make sure our input values are valid, which is between 0 and 1.
			const float ClampedFactor = FMath::Clamp(BoneFactor, 0.0f, 1.0f);

			// Calculate where blend begin value is for this specific bone. So where did our blend start from?
			// Note that this isn't just the BlendInfo.GetBlendedValue() because it can be different per bone as some bones are further ahead in time.
			// We also need to sample the actual curve for this to get the real value.
			const float BeginValue = (ClampedFactor > ZERO_ANIMWEIGHT_THRESH) ? FMath::Clamp(BlendStartAlpha / ClampedFactor, 0.0f, 1.0f) : 1.0f;
			const float RealBeginValue = FAlphaBlend::AlphaToBlendOption(BeginValue, BlendInfo.GetBlendOption(), BlendInfo.GetCustomCurve());

			// Calculate the current alpha value for the bone.
			// As some bones can blend faster than others, we basically scale the current blend's alpha by the bone's factor.
			// After that we sample the curve to get the real alpha blend value.
			const float LinearAlpha = (ClampedFactor > ZERO_ANIMWEIGHT_THRESH) ? FMath::Clamp(BlendInfo.GetAlpha() / ClampedFactor, 0.0f, 1.0f) : 1.0f;
			const float RealBoneAlpha = FAlphaBlend::AlphaToBlendOption(LinearAlpha, BlendInfo.GetBlendOption(), BlendInfo.GetCustomCurve());

			// Now that we know the alpha for our blend, we can calculate the actual weight value.
			// Also make sure the bone weight is valid. Values can't be zero because this could introduce issues during normalization internally in the pipeline.
			const float BoneWeight = RealBeginValue + RealBoneAlpha * (BlendInfo.GetDesiredValue() - RealBeginValue);
			const float ClampedBoneWeight = FMath::Clamp(BoneWeight, ZERO_ANIMWEIGHT_THRESH, 1.0f);

			// Return our calculated weight, depending whether we'd like to invert it or not.
			return !bInverse ? ClampedBoneWeight : (1.0f - ClampedBoneWeight);
		}

		// The per bone value is a factor of the main blend's weight.
		case EBlendProfileMode::WeightFactor:
		{
			if (!bInverse)
			{
				return FMath::Max(MainWeight * BoneFactor, ZERO_ANIMWEIGHT_THRESH);
			}

			// We're inversing.
			const float Weight = (BoneFactor > ZERO_ANIMWEIGHT_THRESH) ? MainWeight / BoneFactor : 1.0f;
			return FMath::Max(Weight, ZERO_ANIMWEIGHT_THRESH);
		}

		// Handle unsupported modes.
		// If you reach this point you have to add another case statement for your newly added blend profile mode.
		default:
		{
			checkf(false, TEXT("The selected Blend Profile Mode is not supported (Mode=%d)"), EnumToUnderlyingType(Mode));
			break;
		}
	}

	return MainWeight;
}

void UBlendProfile::UpdateBoneWeights(FBlendSampleData& InOutCurrentData, const FAlphaBlend& BlendInfo, float BlendStartAlpha, float MainWeight, bool bInverse)
{
	for (int32 PerBoneIndex = 0; PerBoneIndex < InOutCurrentData.PerBoneBlendData.Num(); ++PerBoneIndex)
	{
		InOutCurrentData.PerBoneBlendData[PerBoneIndex] = CalculateBoneWeight(GetEntryBlendScale(PerBoneIndex), Mode, BlendInfo, BlendStartAlpha, MainWeight, bInverse);
	}
}

void UBlendProfile::ClearEntries()
{
	ProfileEntries.Empty();
}