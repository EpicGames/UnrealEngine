// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimSlotEvaluationPose.h"
#include "CoreMinimal.h"
#include "HAL/ThreadSingleton.h"

struct FCompactPose;
struct FBlendedCurve;
namespace UE { namespace Anim { struct FStackAttributeContainer; } }

/**
 * Temporary data that we use during blending of anim montages that use blend profiles.
 * We use this in order to prevent runtime allocations.
 */
struct FBlendProfileScratchData : public TThreadSingleton<FBlendProfileScratchData>
{
	// @todo: TArray<TArray<>> allocations are not preserved during resets. We should either implement a new TArrayOfArrays container
	// or change the deallocation policy perhaps via template specialization?

	TArray<TArray<float>> PerBoneWeights;		// A set of bone weights, per montage instance. Index this like [PoseIndex][CompactPoseBoneIndex].
	TArray<float> PerBoneWeightTotals;			// The bone weight totals for non-additive poses, used for normalizing weights.
	TArray<float> BoneBlendProfileScales;		// The bone profile scale values.

	TArray<FSlotEvaluationPose> Poses;			// Non additive poses.
	TArray<FSlotEvaluationPose> AdditivePoses;	// Additive poses.

	TArray<uint8, TInlineAllocator<8>> PoseIndices;			// The indices inside the PerBoneWeights array, for non additive poses.
	TArray<uint8, TInlineAllocator<8>> AdditivePoseIndices;	// The indices inside the PerBoneWeights array, for additive poses.

	TArray<float, TInlineAllocator<8>> BlendingWeights;					// The per pose blend weights.
	TArray<const FCompactPose*, TInlineAllocator<8>> BlendingPoses;		// The non additive poses to blend.
	TArray<const FBlendedCurve*, TInlineAllocator<8>> BlendingCurves;	// The curves to blend.
	TArray<const UE::Anim::FStackAttributeContainer*, TInlineAllocator<8>> BlendingAttributes;	// The attributes to blend.

	void Reset()
	{
		PerBoneWeights.Reset();
		PerBoneWeightTotals.Reset();
		BoneBlendProfileScales.Reset();
		Poses.Reset();
		AdditivePoses.Reset();
		PoseIndices.Reset();
		AdditivePoseIndices.Reset();
		BlendingWeights.Reset();
		BlendingPoses.Reset();
		BlendingCurves.Reset();
		BlendingAttributes.Reset();
	}

	bool IsEmpty() const
	{
		return PerBoneWeights.IsEmpty() &&
			PerBoneWeightTotals.IsEmpty() &&
			BoneBlendProfileScales.IsEmpty() &&
			Poses.IsEmpty() &&
			AdditivePoses.IsEmpty() &&
			PoseIndices.IsEmpty() &&
			AdditivePoseIndices.IsEmpty() &&
			BlendingWeights.IsEmpty() &&
			BlendingPoses.IsEmpty() &&
			BlendingCurves.IsEmpty() &&
			BlendingAttributes.IsEmpty();
	}
};
