// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BlendSpace.cpp: 2D BlendSpace functionality
=============================================================================*/ 

#include "Animation/BlendSpace.h"
#include "Animation/BlendProfile.h"
#include "Animation/AnimNotifyQueue.h"
#include "AnimationUtils.h"
#include "Animation/BlendSpaceUtilities.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/BlendSpaceHelpers.h"
#include "Animation/BlendSpace1DHelpers.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "Logging/MessageLog.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectThreadContext.h"
#include "Animation/MirrorDataTable.h"

#if WITH_EDITOR
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSpace)

#define LOCTEXT_NAMESPACE "BlendSpace"

// Logs the triangulation/segmentation search to look up the blend samples
//#define DEBUG_LOG_BLENDSPACE_TRIANGULATION

DECLARE_CYCLE_STAT(TEXT("BlendSpace GetAnimPose"), STAT_BlendSpace_GetAnimPose, STATGROUP_Anim);

/** Scratch buffers for multi-threaded usage */
struct FBlendSpaceScratchData : public TThreadSingleton<FBlendSpaceScratchData>
{
	TArray<FBlendSampleData> OldSampleDataList;
	TArray<FBlendSampleData> NewSampleDataList;
	TArray<FGridBlendSample, TInlineAllocator<4> > RawGridSamples;
};

//======================================================================================================================
UBlendSpace::UBlendSpace(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SampleIndexWithMarkers = INDEX_NONE;

	/** Use highest weighted animation as default */
	NotifyTriggerMode = ENotifyTriggerMode::HighestWeightedAnimation;

	if (DimensionIndices.Num() == 0)
	{
		DimensionIndices = { 0, 1 };
	}
}

void UBlendSpace::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR	
	// Only do this during editor time (could alter the blendspace data during runtime otherwise) 
	ValidateSampleData();

	InitializePerBoneBlend();

	if (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::BlendSpaceUpdatedSamplePointAnimationLengthDerivation)
	{
		bUseLegacySamplePointAnimationLengthCalculations = true;
	}
#endif // WITH_EDITOR
}

void UBlendSpace::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);
	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsLoading() && (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::BlendSpacePostLoadSnapToGrid))
	{
		// This will ensure that all grid points are in valid position and the bIsSnapped flag is set
		SnapSamplesToClosestGridPoint();
	}

	if (Ar.IsLoading() && (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::SupportBlendSpaceRateScale))
	{
		for (FBlendSample& Sample : SampleData)
		{
			Sample.RateScale = 1.0f;
		}
	}

	if (Ar.IsLoading() && 
		(Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) <
		 FUE5MainStreamObjectVersion::BlendSpaceRuntimeTriangulation))
	{
		// Make old blend spaces use the grid
		bInterpolateUsingGrid = true;
		// Force the data to be updated
		ResampleData();
		// Preserve the constant sample weight speed for old blend spaces, but allow the ease
		// in/out default for new ones.
		bTargetWeightInterpolationEaseInOut = false;
	}

	if (Ar.IsLoading() && (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) <
		FUE5MainStreamObjectVersion::BlendSpaceSmoothingImprovements))
	{
		// If it's an old asset but has the current default smoothing and that smoothing is in use, then it must
		// have been using the old default so switch to that.
		for (int Index = 0 ; Index != 3 ; ++Index)
		{
			if (InterpolationParam[Index].InterpolationType == EFilterInterpolationType::BSIT_SpringDamper &&
                InterpolationParam[Index].InterpolationTime > 0.0f)
			{
				InterpolationParam[Index].InterpolationType = EFilterInterpolationType::BSIT_Average;
			}
			else if (InterpolationParam[Index].InterpolationType == EFilterInterpolationType::BSIT_Cubic)
			{
				// Cubic was broken since it was not cubing the interpolation time
				InterpolationParam[Index].InterpolationTime = FMath::Pow(
					InterpolationParam[Index].InterpolationTime, 1.0f / 3.0f);
			}
		}
	}

	if (Ar.IsLoading() && 
		(Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) <
		 FUE5MainStreamObjectVersion::BlendSpaceSampleOrdering))
	{
		// Force the data to be updated after the ordering of 2D SampleData has been changed to be consistent with 1D
		ResampleData();
	}

#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UBlendSpace::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	// Cache the axis ranges if it is going to change, this so the samples can be remapped correctly
	const FName PropertyName = PropertyAboutToChange ? PropertyAboutToChange->GetFName() : NAME_None;
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(FBlendParameter, Min) || PropertyName == GET_MEMBER_NAME_CHECKED(FBlendParameter, Max)))
	{
		for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
		{
			PreviousAxisMinMaxValues[AxisIndex].X = BlendParameters[AxisIndex].Min;
			PreviousAxisMinMaxValues[AxisIndex].Y = BlendParameters[AxisIndex].Max;
			PreviousGridSpacings[AxisIndex] = (BlendParameters[AxisIndex].Max - BlendParameters[AxisIndex].Min) / BlendParameters[AxisIndex].GridNum;
		}
	}
}

void UBlendSpace::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UBlendSpace, ManualPerBoneOverrides) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FPerBoneInterpolation, InterpolationSpeedPerSec) ||
		MemberPropertyName == GET_MEMBER_NAME_CHECKED(UBlendSpace, PerBoneBlendProfile) ||
		MemberPropertyName == GET_MEMBER_NAME_CHECKED(UBlendSpace, PerBoneBlendMode) ||
		MemberPropertyName == GET_MEMBER_NAME_CHECKED(UBlendSpace, TargetWeightInterpolationSpeedPerSec))
	{
		InitializePerBoneBlend();
	}

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UBlendSpace, BlendParameters))
	{
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FBlendParameter, GridNum))
		{
			// Tried and snap samples to points on the grid, those who do not fit or cannot be snapped are marked as invalid
			SnapSamplesToClosestGridPoint();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FBlendParameter, Min))
		{
			// Preserve/enforce the previous grid spacing if snapping
			for (int32 AxisIndex = 0; AxisIndex != 2; ++AxisIndex)
			{
				if (BlendParameters[AxisIndex].bSnapToGrid)
				{
					float GridDelta = PreviousGridSpacings[AxisIndex];
					float Range = (BlendParameters[AxisIndex].Max - BlendParameters[AxisIndex].Min);
					BlendParameters[AxisIndex].GridNum = FMath::Max(1, (int32)(0.5f + Range / GridDelta));
					BlendParameters[AxisIndex].Min = BlendParameters[AxisIndex].Max - BlendParameters[AxisIndex].GridNum * GridDelta;
				}
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FBlendParameter, Max))
		{
			// Preserve/enforce the previous grid spacing if snapping
			for (int32 AxisIndex = 0; AxisIndex != 2; ++AxisIndex)
			{
				if (BlendParameters[AxisIndex].bSnapToGrid)
				{
					float GridDelta = PreviousGridSpacings[AxisIndex];
					float Range = (BlendParameters[AxisIndex].Max - BlendParameters[AxisIndex].Min);
					BlendParameters[AxisIndex].GridNum = FMath::Max(1, (int32)(0.5f + Range / GridDelta));
					BlendParameters[AxisIndex].Max = BlendParameters[AxisIndex].Min + BlendParameters[AxisIndex].GridNum * GridDelta;
				}
			}
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

void UBlendSpace::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(this->SampleData.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(this->GridSamples.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(this->BlendSpaceData.StaticStruct()->GetStructureSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(this->BlendSpaceData.Segments.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(this->BlendSpaceData.Triangles.GetAllocatedSize());
}

bool UBlendSpace::UpdateBlendSamples_Internal(
	const FVector&            InBlendSpacePosition, 
	float                     InDeltaTime, 
	TArray<FBlendSampleData>& InOutOldSampleDataList, 
	TArray<FBlendSampleData>& InOutSampleDataCache, 
	int32&                    InOutCachedTriangulationIndex) const
{
	// For Target weight interpolation, we'll need to save old data, and interpolate to new data
	TArray<FBlendSampleData>& NewSampleDataList = FBlendSpaceScratchData::Get().NewSampleDataList;
	check(!NewSampleDataList.Num()); // this must be called non-recursively

	InOutOldSampleDataList.Append(InOutSampleDataCache);

#if WITH_EDITOR
	// If we are in Editor then samples may be added/removed, and when this happens it is not
	// practical to find any existing caches that reference affected animations. Note that all the
	// indices may become invalid. We can just check here for invalid sample indices - if we find one
	// then remove the cache and start again. There will be some situations where the sample indices
	// get shuffled, in which case we will not detect the change, but our cache will now point to
	// incorrect animations. Any glitch that occurs as a result should be removed over the smoothing window. 
	//
	// Note that if this is not sufficient, we could store a GUID in UBlendSpace that gets updated
	// when there is a change, and a GUID in the cache. Then when the GUIDs don't match the cache
	// could be wiped and restarted with the current GUID.
	//
	// See UE-71107
	for (int32 Index = 0; Index < InOutOldSampleDataList.Num(); ++Index)
	{
		if (!SampleData.IsValidIndex(InOutOldSampleDataList[Index].SampleDataIndex))
		{
			InOutOldSampleDataList.Empty();
			break;
		}
	}
#endif

	// get sample data based on new input
	// consolidate all samples and sort them, so that we can handle from biggest weight to smallest
	InOutSampleDataCache.Reset();

	// get sample data from blendspace
	bool bSuccessfullySampled = false;
	if (GetSamplesFromBlendInput(InBlendSpacePosition, NewSampleDataList, InOutCachedTriangulationIndex, true))
	{
		// if target weight interpolation is set
		if (TargetWeightInterpolationSpeedPerSec > 0.f || PerBoneBlendValues.Num() > 0)
		{
			// target weight interpolation
			if (InterpolateWeightOfSampleData(InDeltaTime, InOutOldSampleDataList, NewSampleDataList, InOutSampleDataCache))
			{
				// now I need to normalize
				FBlendSampleData::NormalizeDataWeight(InOutSampleDataCache);
			}
			else
			{
				// if interpolation failed, just copy new sample data to sample data
				InOutSampleDataCache = NewSampleDataList;
			}
		}
		else
		{
			// when there is no target weight interpolation, just copy new to target
			InOutSampleDataCache.Append(NewSampleDataList);
		}

		bSuccessfullySampled = true;
	}

	NewSampleDataList.Reset();

	return bSuccessfullySampled;
}

const UMirrorDataTable* UBlendSpace::GetEffectiveMirrorDataTableForSample(const FAnimTickRecord& InInstance, const FBlendSample& InBlendSample) const
{
	return GetEffectiveMirrorDataTableForSample(InBlendSample, InInstance.MirrorDataTable);
}

const UMirrorDataTable* UBlendSpace::GetEffectiveMirrorDataTableForSample(const FBlendSample& InBlendSample, const UMirrorDataTable* InUpStreamMirrorDataTable) const
{
	const bool bInstanceIsMirrored = InUpStreamMirrorDataTable != nullptr;
	const bool bSampleIsMirrored = InBlendSample.bMirror;

	// We have an upstream mirror request (via graph message), and our sample is unmirrored, use graph message's mirror data table as usual.
	if (bInstanceIsMirrored && !bSampleIsMirrored)
	{
		return InUpStreamMirrorDataTable;
	}

	// No upstream mirroring request (via graph message) therefore we are good to use the blendspace's mirror data table since our sample is mirrored.
	if (!bInstanceIsMirrored && bSampleIsMirrored)
	{
		return MirrorDataTable;
	}

	// We have an upstream mirror request (via graph message) AND we have our sample mirrored. We don't mirror since we need to do the inverse of what the rest of the samples mirror state is (driven by graph message). 
	return nullptr;
}


bool UBlendSpace::UpdateBlendSamples(const FVector& InBlendSpacePosition, float InDeltaTime, TArray<FBlendSampleData>& InOutSampleDataCache, int32& InOutCachedTriangulationIndex) const
{
	TArray<FBlendSampleData>& OldSampleDataList = FBlendSpaceScratchData::Get().OldSampleDataList;
	check(!OldSampleDataList.Num()); // this must be called non-recursively
	const bool bResult = UpdateBlendSamples_Internal(InBlendSpacePosition, InDeltaTime, OldSampleDataList, InOutSampleDataCache, InOutCachedTriangulationIndex);
	OldSampleDataList.Reset();
	return bResult;
}

void UBlendSpace::ResetBlendSamples(TArray<FBlendSampleData>& InOutSampleDataCache, float InNormalizedCurrentTime, bool bLooping, bool bMatchSyncPhases) const
{
	const bool bCanDoMarkerSync = SampleIndexWithMarkers != INDEX_NONE && SampleData.Num() > SampleIndexWithMarkers;

	// Ensure we have a valid normalized time.
	InNormalizedCurrentTime = bLooping ? FMath::Wrap(InNormalizedCurrentTime, 0.0f, 1.0f) : FMath::Clamp(InNormalizedCurrentTime, 0.0f, 1.0f);
	
	// Query highest weighted sample with marker information. This will become the leader for all other samples to follow.
	const int32 HighestMarkerSyncWeightIndex = bCanDoMarkerSync ? FBlendSpaceUtilities::GetHighestWeightMarkerSyncSample(InOutSampleDataCache, SampleData) : INDEX_NONE;
		
	if (HighestMarkerSyncWeightIndex != INDEX_NONE)
	{
		// Query leader sample information.
		FBlendSampleData& LeaderSampleData = InOutSampleDataCache[HighestMarkerSyncWeightIndex];
		const FBlendSample& LeaderSample = SampleData[LeaderSampleData.SampleDataIndex];

		ensure(LeaderSample.Animation != nullptr);

		// Leader is known at this point, build it's tick context.
		FAnimAssetTickContext Context = { 0.0f, ERootMotionMode::NoRootMotionExtraction, true, *LeaderSampleData.Animation->GetUniqueMarkerNames()};
		
		// Reset leader sample to match requested time.
		LeaderSampleData.MarkerTickRecord.Reset();
		LeaderSampleData.PreviousTime = InNormalizedCurrentTime * LeaderSample.GetSamplePlayLength();
		LeaderSampleData.Time = InNormalizedCurrentTime * LeaderSample.GetSamplePlayLength();

		// Query valid marker position for leader.
		LeaderSample.Animation->GetMarkerIndicesForTime(LeaderSampleData.Time, bLooping, Context.MarkerTickContext.GetValidMarkerNames(), LeaderSampleData.MarkerTickRecord.PreviousMarker, LeaderSampleData.MarkerTickRecord.NextMarker);

		// Get leader sync start position.
		if (bMatchSyncPhases)
		{
			FMarkerTickRecord StartMarkerTickRecord;

			// Get sync start position.
			LeaderSample.Animation->GetMarkerIndicesForTime(0, bLooping, Context.MarkerTickContext.GetValidMarkerNames(), StartMarkerTickRecord.PreviousMarker, StartMarkerTickRecord.NextMarker);
			Context.MarkerTickContext.SetMarkerSyncStartPosition(LeaderSample.Animation->GetMarkerSyncPositionFromMarkerIndicies(StartMarkerTickRecord.PreviousMarker.MarkerIndex, StartMarkerTickRecord.NextMarker.MarkerIndex, 0, nullptr));

			// We need to account for an extra passed marker when the LeaderSample is looping and its last marker is placed at PlayLength, since GetMarkerIndicesForTime() will give us Prev = LastMarkerIndex - 1 and Next = LastMarkerIndex when CurrentTime is 0.
			int LeaderLastMarkerIndex = LeaderSampleData.Animation->AuthoredSyncMarkers.Num() - 1;
			if (bLooping && StartMarkerTickRecord.NextMarker.MarkerIndex == LeaderLastMarkerIndex)
			{
				int32 PassedMarker = Context.MarkerTickContext.MarkersPassedThisTick.Add(FPassedMarker());
				Context.MarkerTickContext.MarkersPassedThisTick[PassedMarker].PassedMarkerName = LeaderSampleData.Animation->AuthoredSyncMarkers[LeaderLastMarkerIndex].MarkerName;
				Context.MarkerTickContext.MarkersPassedThisTick[PassedMarker].DeltaTimeWhenPassed = LeaderSampleData.Time;
			}
		}
		else
		{
			Context.MarkerTickContext.SetMarkerSyncStartPosition(LeaderSample.Animation->GetMarkerSyncPositionFromMarkerIndicies(LeaderSampleData.MarkerTickRecord.PreviousMarker.MarkerIndex, LeaderSampleData.MarkerTickRecord.NextMarker.MarkerIndex, LeaderSampleData.Time, nullptr));
		}

		// Get leader sync end position.
		Context.MarkerTickContext.SetMarkerSyncEndPosition(LeaderSample.Animation->GetMarkerSyncPositionFromMarkerIndicies(LeaderSampleData.MarkerTickRecord.PreviousMarker.MarkerIndex, LeaderSampleData.MarkerTickRecord.NextMarker.MarkerIndex, LeaderSampleData.Time, nullptr));

		// Determine how many markers where passed to arrive to the sync phase the leader is currently in.
		if (bMatchSyncPhases)
		{
			FMarkerSyncData SyncData;
			
			SyncData.AuthoredSyncMarkers = LeaderSample.Animation->AuthoredSyncMarkers;
			SyncData.CollectMarkersInRange(0, LeaderSampleData.Time, Context.MarkerTickContext.MarkersPassedThisTick, LeaderSampleData.Time, nullptr);
		}
		
		// Reset follower samples.
		for (int32 SampleIndex = 0; SampleIndex < InOutSampleDataCache.Num(); ++SampleIndex)
		{
			FBlendSampleData& SampleDataItem = InOutSampleDataCache[SampleIndex];
			const FBlendSample& Sample = SampleData[SampleDataItem.SampleDataIndex];
			
			if (HighestMarkerSyncWeightIndex != SampleIndex && (SampleDataItem.TotalWeight > ZERO_ANIMWEIGHT_THRESH))
			{
				if (!Sample.Animation->AuthoredSyncMarkers.IsEmpty())
				{
					// Reset time.
					SampleDataItem.PreviousTime = 0.0f;
					SampleDataItem.Time = 0.0f;
				
					// Get next marker indices that matches sync start position.
					SampleDataItem.MarkerTickRecord.Reset();
					Sample.Animation->GetMarkerIndicesForPosition(Context.MarkerTickContext.GetMarkerSyncStartPosition(), bLooping, SampleDataItem.MarkerTickRecord.PreviousMarker, SampleDataItem.MarkerTickRecord.NextMarker, SampleDataItem.Time, nullptr);

					// Ensure we advance and pass all the phases the leader passed.
					if (bMatchSyncPhases)
					{
						Sample.Animation->AdvanceMarkerPhaseAsFollower(Context.MarkerTickContext, 0.0f, bLooping, SampleDataItem.Time, SampleDataItem.MarkerTickRecord.PreviousMarker, SampleDataItem.MarkerTickRecord.NextMarker, nullptr);
					}
				}
				else
				{
					// Fallback to length based syncing so it matches default behaviour when ticking a blend space.
					SampleDataItem.MarkerTickRecord.Reset();
					SampleDataItem.PreviousTime = InNormalizedCurrentTime * Sample.GetSamplePlayLength();
					SampleDataItem.Time = InNormalizedCurrentTime * Sample.GetSamplePlayLength();
				}
			}
		}
	}
	else
	{
		// Fallback to length based syncing so it matches default behaviour when ticking a blend space.
		for (int32 SampleIndex = 0; SampleIndex < InOutSampleDataCache.Num(); ++SampleIndex)
		{
			FBlendSampleData& SampleDataItem = InOutSampleDataCache[SampleIndex];
			
			SampleDataItem.MarkerTickRecord.Reset();
			SampleDataItem.PreviousTime = InNormalizedCurrentTime * SampleData[SampleDataItem.SampleDataIndex].GetSamplePlayLength();
			SampleDataItem.Time = InNormalizedCurrentTime * SampleData[SampleDataItem.SampleDataIndex].GetSamplePlayLength();
		}
	}
}

void UBlendSpace::ForEachImmutableSample(const TFunctionRef<void(const FBlendSample&)> Func) const
{
	for (const FBlendSample & Sample : SampleData)
	{
		Func(Sample);
	}
}

float UBlendSpace::ComputeAxisScaleFactor(const FVector& BlendSpacePosition, const FVector& FilteredBlendInput) const
{
	const EBlendSpaceAxis AxisToScale = GetAxisToScale();
	if (AxisToScale == BSA_None)
	{
		return 1.0f;
	}

	float FilterMultiplier = 1.f;

	// first use multiplier using new blendinput
	// new filtered input is going to be used for sampling animation
	// so we'll need to change playrate if you'd like to not slide foot
	if (!BlendSpacePosition.Equals(FilteredBlendInput))
	{
		// apply speed change if you want
		if (AxisToScale == BSA_X)
		{
			if (FilteredBlendInput.X != 0.f)
			{
				FilterMultiplier = BlendSpacePosition.X / FilteredBlendInput.X;
			}
		}
		else if (AxisToScale == BSA_Y)
		{
			if (FilteredBlendInput.Y != 0.f)
			{
				FilterMultiplier = BlendSpacePosition.Y / FilteredBlendInput.Y;
			}
		}
	}

	// Now find if clamped input is different. If different, then apply scale to fit in. This allows
	// "extrapolation" of the blend space outside of the range by time scaling the animation, which is
	// appropriate when the specified axis is speed (for example).
	FVector ClampedInput = GetClampedBlendInput(FilteredBlendInput);
	if (!ClampedInput.Equals(FilteredBlendInput))
	{
		// apply speed change if you want, 
		if (AxisToScale == BSA_X && !BlendParameters[0].bWrapInput)
		{
			if (ClampedInput.X != 0.f)
			{
				FilterMultiplier *= FilteredBlendInput.X / ClampedInput.X;
			}
		}
		else if (AxisToScale == BSA_Y)
		{
			if (ClampedInput.Y != 0.f && !BlendParameters[1].bWrapInput)
			{
				FilterMultiplier *= FilteredBlendInput.Y / ClampedInput.Y;
			}
		}
	}

	return FilterMultiplier;
}

void UBlendSpace::TickAssetPlayer(FAnimTickRecord& Instance, struct FAnimNotifyQueue& NotifyQueue, FAnimAssetTickContext& Context) const
{
	check(Instance.BlendSpace.BlendSampleDataCache);

	// Scratch area for old samples
	TArray<FBlendSampleData>& OldSampleDataList = FBlendSpaceScratchData::Get().OldSampleDataList;
	check(!OldSampleDataList.Num()); // this must be called non-recursively
	// new sample data that will be used for evaluation
	TArray<FBlendSampleData>& SampleDataList = *Instance.BlendSpace.BlendSampleDataCache;

	const float DeltaTime = Context.GetDeltaTime();

	check(Instance.DeltaTimeRecord);

	// Our current time will normally be the same as the recorded previous time, if we were simply playing animations. 
	// If it's not, then that would be because the time has been explicitly set due to be run as an evaluator, in 
	// which case we might want to use that delta in addition to any time delta due to the play rate.
	float ExtraNormalizedDeltaTime = 
		Instance.DeltaTimeRecord->IsPreviousValid() ?
		FMath::Wrap(*(Instance.TimeAccumulator) - Instance.DeltaTimeRecord->GetPrevious(), -0.5f, 0.5f) :
		0.0f;

	// The blend space delta time record isn't normally used directly.
	// Instead, the blend sample time record's are used to drive pose evaluation
	// As a consequence, we currently follow the convention that TimeAccumulator and Previous are normalized
	Instance.DeltaTimeRecord->Set(*(Instance.TimeAccumulator), Instance.PlayRateMultiplier * DeltaTime); 

	// this happens even if MoveDelta == 0.f. This still should happen if it is being interpolated
	// since we allow setting position of blendspace, we can't ignore MoveDelta == 0.f
	// also now we don't have to worry about not following if DeltaTime = 0.f

	// first filter input using blend filter
	const FVector BlendSpacePosition(Instance.BlendSpace.BlendSpacePositionX, Instance.BlendSpace.BlendSpacePositionY, 0.f);
	const FVector FilteredBlendInput = FilterInput(Instance.BlendSpace.BlendFilter, BlendSpacePosition, DeltaTime);

	if (UpdateBlendSamples_Internal(FilteredBlendInput, DeltaTime, OldSampleDataList, SampleDataList, Instance.BlendSpace.TriangulationIndex))
	{
		float NewAnimLength = 0.f;
		float PreInterpAnimLength = 0.f;

		if (TargetWeightInterpolationSpeedPerSec > 0.f)
		{
			// recalculate AnimLength based on weight of target animations - this is used for scaling animation later (change speed)
			PreInterpAnimLength = GetAnimationLengthFromSampleData(*Instance.BlendSpace.BlendSampleDataCache);
			UE_LOGF(LogAnimation, Verbose, "BlendSpace(%ls) - FilteredBlendInput(%ls) : PreAnimLength(%0.5f) ", *GetName(), *FilteredBlendInput.ToString(), PreInterpAnimLength);
		}

		float FilterMultiplier = ComputeAxisScaleFactor(BlendSpacePosition, FilteredBlendInput);
		if (FilterMultiplier != 1.0f)
		{
			Instance.DeltaTimeRecord->Delta *= FilterMultiplier;
			UE_LOGF(LogAnimation, Verbose, "BlendSpace(%ls) - FilteredBlendInput(%ls) : FilteredBlendInput(%ls), FilterMultiplier(%0.2f)",
				*GetName(), *BlendSpacePosition.ToString(), *FilteredBlendInput.ToString(), FilterMultiplier);
		}

		// We can use marker-based syncing when a valid sample with sync marker data exists.
		bool bCanDoMarkerSync = (SampleIndexWithMarkers != INDEX_NONE) && (Context.IsSingleAnimationContext() || (Instance.bCanUseMarkerSync && Context.CanUseMarkerPosition()));
			
		if (bCanDoMarkerSync)
		{
			// Copy previous frame marker data to current frame
			for (const FBlendSampleData& PrevBlendSampleItem : OldSampleDataList)
			{
				for (FBlendSampleData& CurrentBlendSampleItem : SampleDataList)
				{
					// it only can have one animation in the sample, make sure to copy Time
					if (PrevBlendSampleItem.Animation && PrevBlendSampleItem.Animation == CurrentBlendSampleItem.Animation)
					{
						CurrentBlendSampleItem.Time = PrevBlendSampleItem.Time;
						CurrentBlendSampleItem.PreviousTime = PrevBlendSampleItem.PreviousTime;
						CurrentBlendSampleItem.MarkerTickRecord = PrevBlendSampleItem.MarkerTickRecord;
					}
				}
			}
		}

		NewAnimLength = GetAnimationLengthFromSampleData(SampleDataList);

		if (PreInterpAnimLength > 0.f && NewAnimLength > 0.f)
		{
			Instance.DeltaTimeRecord->Delta *= PreInterpAnimLength / NewAnimLength;
		}

		float& NormalizedCurrentTime = *(Instance.TimeAccumulator);
		if (Context.ShouldResyncToSyncGroup() && !Instance.bIsEvaluator)
		{
			// Synchronize the asset player time to the other sync group members when (re)joining the group
			NormalizedCurrentTime = Context.GetAnimationPositionRatio();
		}

		float NormalizedPreviousTime = NormalizedCurrentTime;

		// @note for sync group vs non sync group
		// in blendspace, it will still sync even if only one node in sync group
		// so you're never non-sync group unless you have situation where some markers are relevant to one sync group but not all the time
		// here we save NormalizedCurrentTime as Highest weighted samples' position in sync group
		// if you're not in sync group, NormalizedCurrentTime is based on normalized length by sample weights
		// if you move between sync to non sync within blendspace, you're going to see pop because we'll have to jump
		// for now, our rule is to keep normalized time as highest weighted sample position within its own length
		// also MoveDelta doesn't work if you're in sync group. It will move according to sync group position
		// @todo consider using MoveDelta when  this is leader, but that can be scary because it's not matching with DeltaTime any more. 
		// if you have interpolation delay, that value can be applied, but the output might be unpredictable. 
		// 
		// to fix this better in the future, we should use marker sync position from last tick
		// but that still doesn't fix if you just join sync group, you're going to see pop since your animation doesn't fix

		if (Context.IsLeader())
		{
			// advance current time - blend spaces hold normalized time as when dealing with changing anim length it would be possible to go backwards
			UE_LOGF(LogAnimation, Verbose, "BlendSpace(%ls) - FilteredBlendInput(%ls) : AnimLength(%0.5f) ", *GetName(), *FilteredBlendInput.ToString(), NewAnimLength);

			// Set context's data before updating time position.
			Context.SetPreviousAnimationPositionRatio(NormalizedCurrentTime);

			// Get highest weight sample with sync markers. This will become the leader for all other samples to follow.
			const int32 HighestMarkerSyncWeightIndex = bCanDoMarkerSync ? FBlendSpaceUtilities::GetHighestWeightMarkerSyncSample(SampleDataList, SampleData) : INDEX_NONE;

			// Skip syncing, fallback to normal ticking.
			if (HighestMarkerSyncWeightIndex == INDEX_NONE)
			{
				bCanDoMarkerSync = false;
			}

			// Tick as leader using marked based syncing.
			if (bCanDoMarkerSync)
			{
				if (bShouldMatchSyncPhases)
				{
					FBlendSampleData& LeaderSampleData = SampleDataList[HighestMarkerSyncWeightIndex];
					const FBlendSample& LeaderSample = SampleData[LeaderSampleData.SampleDataIndex];
					
					if (LeaderSample.Animation)
					{
						// Update leader delta.
						float NewDelta = Instance.DeltaTimeRecord->Delta * LeaderSample.RateScale * LeaderSample.Animation->RateScale;
						Context.SetLeaderDelta(NewDelta);

						// Get mirror table.
						const UMirrorDataTable* LeaderMirrorDataTable = GetEffectiveMirrorDataTableForSample(Instance, LeaderSample);

						// Advance time as usual.
						{
							float CurrentTime = NormalizedCurrentTime * NewAnimLength;
                        	LeaderSampleData.PreviousTime = CurrentTime;
                        	
                        	FAnimationRuntime::AdvanceTime(Instance.bLooping, NewDelta, /*inout*/ CurrentTime, NewAnimLength);
                        	
                        	LeaderSampleData.Time = CurrentTime;
                        	NormalizedCurrentTime = NewAnimLength ? (CurrentTime / NewAnimLength) : 0.0f;
						}

						// Sync position BEFORE advancing time.
						LeaderSample.Animation->GetMarkerIndicesForTime(LeaderSampleData.PreviousTime, Instance.bLooping, Context.MarkerTickContext.GetValidMarkerNames(), LeaderSampleData.MarkerTickRecord.PreviousMarker, LeaderSampleData.MarkerTickRecord.NextMarker);
						const FMarkerSyncAnimPosition StartSyncPosition = LeaderSample.Animation->GetMarkerSyncPositionFromMarkerIndicies(LeaderSampleData.MarkerTickRecord.PreviousMarker.MarkerIndex, LeaderSampleData.MarkerTickRecord.NextMarker.MarkerIndex, LeaderSampleData.PreviousTime, LeaderMirrorDataTable);
						Context.MarkerTickContext.SetMarkerSyncStartPosition(StartSyncPosition);

						// @todo: See if we can have reset blend samples directly update the context and/or tick record (no need to compute info twice, if possible). 
						// Rest all samples to match current time.
						ResetBlendSamples(SampleDataList, NormalizedCurrentTime, Instance.bLooping, true);

						// Sync position AFTER advancing time.
						LeaderSample.Animation->GetMarkerIndicesForTime(LeaderSampleData.Time, Instance.bLooping, Context.MarkerTickContext.GetValidMarkerNames(), LeaderSampleData.MarkerTickRecord.PreviousMarker, LeaderSampleData.MarkerTickRecord.NextMarker);
						*Instance.MarkerTickRecord = LeaderSampleData.MarkerTickRecord;
						const FMarkerSyncAnimPosition EndSyncPosition = LeaderSample.Animation->GetMarkerSyncPositionFromMarkerIndicies(LeaderSampleData.MarkerTickRecord.PreviousMarker.MarkerIndex, LeaderSampleData.MarkerTickRecord.NextMarker.MarkerIndex, LeaderSampleData.Time, LeaderMirrorDataTable);
						Context.MarkerTickContext.SetMarkerSyncEndPosition(EndSyncPosition);

						// Update context's position ratios
						Context.SetAnimationPositionRatio(NormalizedCurrentTime);
						Context.SetPreviousAnimationPositionRatio(NormalizedPreviousTime);
						
						// Keep track of the markers we passed.
						FMarkerSyncData SyncData;
						SyncData.AuthoredSyncMarkers = LeaderSample.Animation->AuthoredSyncMarkers;
						SyncData.CollectMarkersInRange(LeaderSampleData.PreviousTime, LeaderSampleData.Time, Context.MarkerTickContext.MarkersPassedThisTick, NewDelta, LeaderMirrorDataTable);
					}
				}
				else
				{
					FBlendSampleData& LeaderSampleData = SampleDataList[HighestMarkerSyncWeightIndex];
					const FBlendSample& LeaderSample = SampleData[LeaderSampleData.SampleDataIndex];

					if (LeaderSample.Animation)
					{
						bool bResetMarkerDataOnFollowers = false;
						const UMirrorDataTable* LeaderMirrorDataTable = GetEffectiveMirrorDataTableForSample(Instance, LeaderSample);
						
						// Invalidate sample followers' tick records if instance doesn't have any valid sync marker data. 
						if (!Instance.MarkerTickRecord->IsValid(Instance.bLooping))
						{
							LeaderSampleData.MarkerTickRecord.Reset();
							LeaderSampleData.Time = NormalizedCurrentTime * LeaderSample.GetSamplePlayLength();
							bResetMarkerDataOnFollowers = true;
						}
						// Re-compute marker indices since the leader sample's tick record is invalid. Get previous and next markers.
						else if (!LeaderSampleData.MarkerTickRecord.IsValid(Instance.bLooping) && Context.MarkerTickContext.GetMarkerSyncStartPosition().IsValid())
						{
							// TODO: Look into the reason for not passing bLooping variable and just forcing the vale to be true. 
							LeaderSample.Animation->GetMarkerIndicesForPosition(Context.MarkerTickContext.GetMarkerSyncStartPosition(), true, LeaderSampleData.MarkerTickRecord.PreviousMarker, LeaderSampleData.MarkerTickRecord.NextMarker, LeaderSampleData.Time, LeaderMirrorDataTable);
						}

						// Only tick samples if leader sample has any delta time to consume.
						const float NewDeltaTime = Instance.DeltaTimeRecord->Delta * LeaderSample.RateScale * LeaderSample.Animation->RateScale;
						Context.SetLeaderDelta(NewDeltaTime);

						if (!FMath::IsNearlyZero(NewDeltaTime))
						{
							// Tick leader sample
							LeaderSample.Animation->TickByMarkerAsLeader(LeaderSampleData.MarkerTickRecord, Context.MarkerTickContext, LeaderSampleData.Time, LeaderSampleData.PreviousTime, NewDeltaTime, Instance.bLooping, LeaderMirrorDataTable);

							check(!Instance.bLooping || Context.MarkerTickContext.IsMarkerSyncStartValid());
							
							// Tick all the follower samples
							TickFollowerSamples(SampleDataList, HighestMarkerSyncWeightIndex, Context, bResetMarkerDataOnFollowers, Instance.bLooping, Instance);
						}
						else if (!Instance.MarkerTickRecord->IsValid(Instance.bLooping))
						{
							// Re-compute marker indices for leader sample's tick record. Get previous and next markers.
							LeaderSample.Animation->GetMarkerIndicesForTime(LeaderSampleData.Time, Instance.bLooping, Context.MarkerTickContext.GetValidMarkerNames(), LeaderSampleData.MarkerTickRecord.PreviousMarker, LeaderSampleData.MarkerTickRecord.NextMarker);

							// Get sync position for followers to sync up to.
							const FMarkerSyncAnimPosition SyncPosition = LeaderSample.Animation->GetMarkerSyncPositionFromMarkerIndicies(LeaderSampleData.MarkerTickRecord.PreviousMarker.MarkerIndex, LeaderSampleData.MarkerTickRecord.NextMarker.MarkerIndex, LeaderSampleData.Time, LeaderMirrorDataTable);
							Context.MarkerTickContext.SetMarkerSyncStartPosition(SyncPosition);
							Context.MarkerTickContext.SetMarkerSyncEndPosition(SyncPosition);
							
							// Make all follower samples match next sync position to equal that of the leader.
							TickFollowerSamples(SampleDataList, HighestMarkerSyncWeightIndex, Context, true, Instance.bLooping, Instance);
						}
						
						NormalizedCurrentTime = LeaderSampleData.Time / LeaderSample.GetSamplePlayLength();
						*Instance.MarkerTickRecord = LeaderSampleData.MarkerTickRecord;
					}
				}
			}
			else
			{
				// Advance time using current/new anim length
				float CurrentTime = NormalizedCurrentTime * NewAnimLength;
				FAnimationRuntime::AdvanceTime(Instance.bLooping, Instance.DeltaTimeRecord->Delta, /*inout*/ CurrentTime, NewAnimLength);
				NormalizedCurrentTime = NewAnimLength ? (CurrentTime / NewAnimLength) : 0.0f;
				UE_LOGF(LogAnimMarkerSync, Log, 
					"Leader (%ls) (bCanDoMarkerSync == false)  - PreviousTime (%0.2f), CurrentTime (%0.2f), MoveDelta (%0.2f) ", 
					*GetName(), NormalizedPreviousTime, NormalizedCurrentTime, Instance.DeltaTimeRecord->Delta);
			}

			// Update time position after it has undergone all side effects.
			Context.SetAnimationPositionRatio(NormalizedCurrentTime);
		}
		else
		{
			// Skip syncing if leader doesn't have a valid sync start position.
			if (!Context.MarkerTickContext.IsMarkerSyncStartValid())
			{
				bCanDoMarkerSync = false;
			}

			// Tick as follower using marked-based syncing.
			if (bCanDoMarkerSync)
			{
				const int32 HighestWeightIndex = FBlendSpaceUtilities::GetHighestWeightSample(SampleDataList);
				FBlendSampleData& SampleDataItem = SampleDataList[HighestWeightIndex];
				const FBlendSample& Sample = SampleData[SampleDataItem.SampleDataIndex];

				if (Sample.Animation)
				{
					// Only tick samples if sync group leader has any delta time to consume.
					if (Context.GetDeltaTime() != 0.f)
					{
						if (!Instance.MarkerTickRecord->IsValid(Instance.bLooping))
						{
							SampleDataItem.Time = NormalizedCurrentTime * Sample.GetSamplePlayLength();
						}

						// Tick all samples as followers
						TickFollowerSamples(SampleDataList, INDEX_NONE, Context, false, Instance.bLooping, Instance);
					}
						
					*Instance.MarkerTickRecord = SampleDataItem.MarkerTickRecord;
					NormalizedCurrentTime = SampleDataItem.Time / Sample.GetSamplePlayLength();
				}
			}
			else
			{
				// Fallback to length-based syncing. Match sync group leader position.
				NormalizedPreviousTime = Context.GetPreviousAnimationPositionRatio();
				NormalizedCurrentTime = Context.GetAnimationPositionRatio();
					
				UE_LOGF(LogAnimMarkerSync, Log, 
					"Follower (%ls) (bCanDoMarkerSync == false) - PreviousTime (%0.2f), CurrentTime (%0.2f), MoveDelta (%0.2f) ", 
					*GetName(), NormalizedPreviousTime, NormalizedCurrentTime, Instance.DeltaTimeRecord->Delta);
			}
		}

		// Generate notifies and sets time.
		{
			FAnimNotifyContext NotifyContext(Instance);
			float ClampedNormalizedPreviousTime = FMath::Clamp<float>(NormalizedPreviousTime, 0.f, 1.f);
			float ClampedNormalizedCurrentTime = FMath::Clamp<float>(NormalizedCurrentTime, 0.f, 1.f);

			if (Instance.bIsEvaluator && !Instance.BlendSpace.bTeleportToTime)
			{
				// When running under an evaluator the time is being set explicitly and we want to add on the deltas.
				ClampedNormalizedPreviousTime -= ExtraNormalizedDeltaTime;
				// Note that ExtraNormalizedDeltaTime can be negative
				ClampedNormalizedPreviousTime = FMath::Wrap<float>(ClampedNormalizedPreviousTime, 0.0f, 1.0f);

				// Also when under an evaluator, since the time is explicitly set before the update is called, the desired 
				// current time is actually what we recorded before advancing time (effectively ignoring whatever was added).
				ClampedNormalizedCurrentTime = FMath::Clamp<float>(NormalizedPreviousTime, 0.f, 1.f);
			}

			const bool bHasDeltaTime = (NormalizedCurrentTime != NormalizedPreviousTime);
			const bool bGenerateNotifies = NotifyTriggerMode != ENotifyTriggerMode::None;

			// Get the index of the highest weight, assuming that the first is the highest until we find otherwise
			const bool bTriggerNotifyHighestWeightedAnim = NotifyTriggerMode == ENotifyTriggerMode::HighestWeightedAnimation && SampleDataList.Num() > 0;
			const int32 HighestWeightIndex = (bGenerateNotifies && bTriggerNotifyHighestWeightedAnim) ? FBlendSpaceUtilities::GetHighestWeightSample(SampleDataList) : INDEX_NONE;

			for (int32 I = 0; I < SampleDataList.Num(); ++I)
			{
				FBlendSampleData& SampleEntry = SampleDataList[I];
				const int32 SampleDataIndex = SampleEntry.SampleDataIndex;

				// Skip SamplesPoints that has no relevant weight
				if (SampleData.IsValidIndex(SampleDataIndex) && (SampleEntry.TotalWeight > ZERO_ANIMWEIGHT_THRESH))
				{
					const FBlendSample& Sample = SampleData[SampleDataIndex];
					if (Sample.Animation)
					{
						float PrevSampleDataTime;
						float& CurrentSampleDataTime = SampleEntry.Time;

						const float MultipliedSampleRateScale = Sample.Animation->RateScale * Sample.RateScale;

						if (!bCanDoMarkerSync || Sample.Animation->AuthoredSyncMarkers.Num() == 0) //Have already updated time if we are doing marker sync
						{
							const float SampleNormalizedPreviousTime = MultipliedSampleRateScale >= 0.f ? ClampedNormalizedPreviousTime : 1.f - ClampedNormalizedPreviousTime;
							const float SampleNormalizedCurrentTime = MultipliedSampleRateScale >= 0.f ? ClampedNormalizedCurrentTime : 1.f - ClampedNormalizedCurrentTime;
							if (Sample.bUseSingleFrameForBlending)
							{
								PrevSampleDataTime = CurrentSampleDataTime = Sample.Animation->GetTimeAtFrame(Sample.FrameIndexToSample);
							}
							else
							{
								PrevSampleDataTime = SampleNormalizedPreviousTime * Sample.GetSamplePlayLength();
								CurrentSampleDataTime = SampleNormalizedCurrentTime * Sample.GetSamplePlayLength();
							}
						}
						else
						{
							PrevSampleDataTime = SampleEntry.PreviousTime;
						}

						// Figure out delta time 
						float DeltaTimePosition = CurrentSampleDataTime - PrevSampleDataTime;
						const float SampleMoveDelta = Instance.DeltaTimeRecord->Delta * MultipliedSampleRateScale;

						// if we went against play rate, then loop around.
						if ((SampleMoveDelta * DeltaTimePosition) < 0.f)
						{
							DeltaTimePosition += FMath::Sign<float>(SampleMoveDelta) * Sample.GetSamplePlayLength();
						}

						if (bGenerateNotifies && (!bTriggerNotifyHighestWeightedAnim || (I == HighestWeightIndex)))
						{
							// Harvest and record notifies
							Sample.Animation->GetAnimNotifies(PrevSampleDataTime, DeltaTimePosition, NotifyContext);
						}

						if (bHasDeltaTime)
						{
							if (Context.RootMotionMode == ERootMotionMode::RootMotionFromEverything && Sample.Animation->bEnableRootMotion)
							{
								const FAnimExtractContext ExtractionContext(static_cast<double>(PrevSampleDataTime), true, FDeltaTimeRecord(DeltaTimePosition), Instance.bLooping);
								Context.RootMotionMovementParams.AccumulateWithBlend(Sample.Animation->ExtractRootMotion(ExtractionContext), SampleEntry.GetClampedWeight());
							}
						}

						// Capture the final adjusted delta time and previous frame time as an asset player record
						SampleEntry.DeltaTimeRecord.Set(PrevSampleDataTime, DeltaTimePosition);

						UE_LOGF(LogAnimation, Verbose, "%d. Blending animation(%ls) with %f weight at time %0.2f", I + 1, *Sample.Animation->GetName(), SampleEntry.GetClampedWeight(), CurrentSampleDataTime);
					}
				}
			}

			if (bGenerateNotifies && NotifyContext.ActiveNotifies.Num() > 0)
			{
				NotifyQueue.AddAnimNotifies(Context.ShouldGenerateNotifies(), NotifyContext.ActiveNotifies, Instance.EffectiveBlendWeight);
			}
		}
	}

	OldSampleDataList.Reset();
}

bool UBlendSpace::IsValidAdditive() const
{
	return ContainsMatchingSamples(AAT_LocalSpaceBase) || ContainsMatchingSamples(AAT_RotationOffsetMeshSpace);
}

#if WITH_EDITOR
bool UBlendSpace::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive /*= true*/)
{
	Super::GetAllAnimationSequencesReferred(AnimationAssets, bRecursive);

	for (auto Iter = SampleData.CreateConstIterator(); Iter; ++Iter)
	{
		// saves all samples in the AnimSequences
		UAnimSequence* Sequence = (*Iter).Animation;
		if (Sequence)
		{
			Sequence->HandleAnimReferenceCollection(AnimationAssets, bRecursive);
		}
	}

	if (PreviewBasePose)
	{
		PreviewBasePose->HandleAnimReferenceCollection(AnimationAssets, bRecursive);
	}

	return (AnimationAssets.Num() > 0);
}

void UBlendSpace::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	Super::ReplaceReferredAnimations(ReplacementMap);

	TArray<FBlendSample> NewSamples;
	for (FBlendSample& Sample : SampleData)
	{
		// replace the referenced animation sequence (if there was one)
		if (Sample.Animation)
		{
			if (UAnimationAsset* const* ReplacementAsset = ReplacementMap.Find(Sample.Animation))
			{
				if (UAnimSequence* ReplacementAnimSequence = Cast<UAnimSequence>(*ReplacementAsset))
				{
					Sample.Animation = ReplacementAnimSequence;
					Sample.Animation->ReplaceReferredAnimations(ReplacementMap);
				}
			}
		}

		NewSamples.Add(Sample);
	}

	// replace preview base pose sequence asset (if there was one)
	if (PreviewBasePose)
	{
		if (UAnimationAsset* const* ReplacementAsset = ReplacementMap.Find(PreviewBasePose))
		{
			if (UAnimSequence* ReplacementAnimSequence = Cast<UAnimSequence>(*ReplacementAsset))
			{
				PreviewBasePose = ReplacementAnimSequence;
				PreviewBasePose->ReplaceReferredAnimations(ReplacementMap);
			}
		}
	}

	SampleData = NewSamples;
}

int32 UBlendSpace::GetMarkerUpdateCounter() const
{
	return MarkerDataUpdateCounter;
}

void UBlendSpace::RuntimeValidateMarkerData()
{
	check(IsInGameThread());

	for (FBlendSample& Sample : SampleData)
	{
		if (Sample.Animation && Sample.CachedMarkerDataUpdateCounter != Sample.Animation->GetMarkerUpdateCounter())
		{
			// Revalidate data
			ValidateSampleData();
			return;
		}
	}
}

#endif // WITH_EDITOR

TConstArrayView<FAnimSyncMarker> UBlendSpace::GetAuthoredSyncMarkers() const
{
	if (SampleIndexWithMarkers != INDEX_NONE && SampleData.Num() > SampleIndexWithMarkers)
	{
		const FBlendSample& BlendSample = SampleData[SampleIndexWithMarkers];
		if (BlendSample.Animation != nullptr)
		{
			return MakeConstArrayView(BlendSample.Animation->AuthoredSyncMarkers);
		}
	}

	return TConstArrayView<FAnimSyncMarker>(); 
}

/** When per-bone blend data are required to be sorted, this stores the sorted copy and the index of the original */
struct FSortedPerBoneInterpolation
{
	FSortedPerBoneInterpolation(const FPerBoneInterpolation& Original, int32 Index)
		: PerBoneBlend(Original)
		, OriginalIndex(Index)
	{
	}

	FPerBoneInterpolation PerBoneBlend;

	/** Index into the original array */
	int32 OriginalIndex;
};

struct FSortedPerBoneInterpolationData : public IInterpolationIndexProvider::FPerBoneInterpolationData
{
	explicit FSortedPerBoneInterpolationData(const FSkeletonRemapping& InSkeletonMapping)
		: SkeletonMapping(InSkeletonMapping)
	{
	}

	const FSkeletonRemapping& SkeletonMapping;
	TArray<FSortedPerBoneInterpolation> Data;
};

TSharedPtr<IInterpolationIndexProvider::FPerBoneInterpolationData> UBlendSpace::GetPerBoneInterpolationData(const USkeleton* RuntimeSkeleton) const 
{
	const USkeleton* SourceSkeleton = GetSkeleton();
	const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry().Get().GetRemapping(SourceSkeleton, RuntimeSkeleton);

	FSortedPerBoneInterpolationData* Data = new FSortedPerBoneInterpolationData(SkeletonRemapping);
	Data->Data.SetNumUninitialized(PerBoneBlendValues.Num());

	for (int32 Iter = 0 ; Iter != PerBoneBlendValues.Num() ; ++Iter)
	{
		Data->Data[Iter] = FSortedPerBoneInterpolation(PerBoneBlendValues[Iter], Iter);
		if (!IsAsset())
		{
			FBoneReference& Bone = Data->Data[Iter].PerBoneBlend.BoneReference;
			// Note that for blendspace graphs, the bone index won't have been set since the skeleton wasn't known at
			// creation time. In that case, look it up (but note that it could be slow).
			Bone.Initialize(RuntimeSkeleton);
		}
	}

	Data->Data.Sort([](const FSortedPerBoneInterpolation& A, const FSortedPerBoneInterpolation& B)
					{ return A.PerBoneBlend.BoneReference.BoneIndex > B.PerBoneBlend.BoneReference.BoneIndex; });
	return MakeShareable<FSortedPerBoneInterpolationData>(Data);
}

int32 UBlendSpace::GetPerBoneInterpolationIndex(
	const FCompactPoseBoneIndex&                                  InCompactPoseBoneIndex, 
	const FBoneContainer&                                         RequiredBones, 
	const IInterpolationIndexProvider::FPerBoneInterpolationData* InData) const
{
	if (!ensure(InData))
	{
		return INDEX_NONE;
	}

	const FSortedPerBoneInterpolationData* Data = static_cast<const FSortedPerBoneInterpolationData*>(InData);
	const TArray<FSortedPerBoneInterpolation>& SortedData = Data->Data;

	const USkeleton* SourceSkeleton = GetSkeleton();
	const FSkeletonRemapping& SkeletonRemapping = Data->SkeletonMapping;

	for (int32 Iter = 0; Iter < SortedData.Num(); ++Iter)
	{
		const FPerBoneInterpolation& PerBoneInterpolation = SortedData[Iter].PerBoneBlend;
		const int32 OriginalIndex = SortedData[Iter].OriginalIndex;
		const FBoneReference& SmoothedBone = PerBoneInterpolation.BoneReference;

		FSkeletonPoseBoneIndex SkelBoneIndex = SmoothedBone.GetSkeletonPoseIndex(RequiredBones);

		// Remap to the target skeleton, using skeleton remapping, as we might be applying this blend space onto another skeleton than the asset was created for.
		if (SkeletonRemapping.IsValid())
		{
			const int32 RemappedSkelBoneIndex = SkeletonRemapping.GetTargetSkeletonBoneIndex(SkelBoneIndex.GetInt());
			SkelBoneIndex = FSkeletonPoseBoneIndex(RemappedSkelBoneIndex);
		}

		const FCompactPoseBoneIndex SmoothedBoneCompactPoseIndex = RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(SkelBoneIndex);
		if (SmoothedBoneCompactPoseIndex == InCompactPoseBoneIndex)
		{
			return OriginalIndex;
		}

		// BoneIsChildOf returns true if InCompactPoseBoneIndex is a child of SmoothedBoneCompactPoseIndex. 
		if (SmoothedBoneCompactPoseIndex != INDEX_NONE && 
			RequiredBones.BoneIsChildOf(InCompactPoseBoneIndex, SmoothedBoneCompactPoseIndex))
		{
			return OriginalIndex;
		}
	}
	return INDEX_NONE;
}

namespace UE::Anim::Private
{
	static FBoneContainer DummyContainer;
}

int32 UBlendSpace::GetPerBoneInterpolationIndex(
	const FSkeletonPoseBoneIndex InSkeletonBoneIndex,
	const USkeleton* TargetSkeleton,
	const IInterpolationIndexProvider::FPerBoneInterpolationData* InData) const
{
	if (!ensure(InData) || !InSkeletonBoneIndex.IsValid())
	{
		return INDEX_NONE;
	}

	const FSortedPerBoneInterpolationData* Data = static_cast<const FSortedPerBoneInterpolationData*>(InData);
	const TArray<FSortedPerBoneInterpolation>& SortedData = Data->Data;

	const USkeleton* SourceSkeleton = GetSkeleton();
	const FReferenceSkeleton& TargetReferenceSkeleton = TargetSkeleton->GetReferenceSkeleton();
	const FSkeletonRemapping& SkeletonRemapping = Data->SkeletonMapping;

	for (const FSortedPerBoneInterpolation& SortedBoneData : SortedData)
	{
		const FPerBoneInterpolation& PerBoneInterpolation = SortedBoneData.PerBoneBlend;
		const FBoneReference& SmoothedBone = PerBoneInterpolation.BoneReference;

		// Bone references are stored as skeleton bones, we can use a dummy bone container as it isn't required
		FSkeletonPoseBoneIndex SmoothedSkelBoneIndex = SmoothedBone.GetSkeletonPoseIndex(UE::Anim::Private::DummyContainer);

		// Remap to the target skeleton, using skeleton remapping, as we might be applying this blend space onto another skeleton than the asset was created for.
		if (SkeletonRemapping.IsValid())
		{
			const int32 RemappedSkelBoneIndex = SkeletonRemapping.GetTargetSkeletonBoneIndex(SmoothedSkelBoneIndex.GetInt());
			SmoothedSkelBoneIndex = FSkeletonPoseBoneIndex(RemappedSkelBoneIndex);
		}

		if (SmoothedSkelBoneIndex == InSkeletonBoneIndex)
		{
			return SortedBoneData.OriginalIndex;
		}

		// BoneIsChildOf returns true if InSkeletonBoneIndex is a child of SmoothedSkelBoneIndex
		if (SmoothedSkelBoneIndex.IsValid() &&
			TargetReferenceSkeleton.BoneIsChildOf(InSkeletonBoneIndex.GetInt(), SmoothedSkelBoneIndex.GetInt()))
		{
			return SortedBoneData.OriginalIndex;
		}
	}

	return INDEX_NONE;
}

bool UBlendSpace::IsValidAdditiveType(EAdditiveAnimationType AdditiveType) const
{
	return (AdditiveType == AAT_LocalSpaceBase || AdditiveType == AAT_RotationOffsetMeshSpace || AdditiveType == AAT_None);
}

void UBlendSpace::ResetToRefPose(FCompactPose& OutPose) const
{
	if (IsValidAdditive())
	{
		OutPose.ResetToAdditiveIdentity();
	}
	else
	{
		OutPose.ResetToRefPose();
	}
}

static const FAnimExtractContext DefaultBlendSpaceExtractionContext = { 0.0, true, {}, false };

void UBlendSpace::GetAnimationPose(TArray<FBlendSampleData>& BlendSampleDataCache, /*out*/ FCompactPose& OutPose, /*out*/ FBlendedCurve& OutCurve) const
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData = { OutPose, OutCurve, TempAttributes };
	GetAnimationPose(BlendSampleDataCache, DefaultBlendSpaceExtractionContext, AnimationPoseData);
}

void UBlendSpace::GetAnimationPose(TArray<FBlendSampleData>& BlendSampleDataCache, const FAnimExtractContext& ExtractionContext, /*out*/ FAnimationPoseData& OutAnimationPoseData) const
{
	GetAnimationPose_Internal(BlendSampleDataCache, TArrayView<FPoseLink>(), nullptr, false, ExtractionContext, OutAnimationPoseData);
}

void UBlendSpace::GetAnimationPose(TArray<FBlendSampleData>& BlendSampleDataCache, TArrayView<FPoseLink> InPoseLinks, const FAnimExtractContext& ExtractionContext, /*out*/ FPoseContext& Output) const
{
	FAnimationPoseData AnimationPoseData(Output);
	GetAnimationPose_Internal(BlendSampleDataCache, InPoseLinks, Output.AnimInstanceProxy, Output.ExpectsAdditivePose(), ExtractionContext, AnimationPoseData);
}

void UBlendSpace::GetAnimationPose_Internal(TArray<FBlendSampleData>& BlendSampleDataCache, TArrayView<FPoseLink> InPoseLinks, FAnimInstanceProxy* InProxy, bool bInExpectsAdditivePose, const FAnimExtractContext& ExtractionContext, /*out*/ FAnimationPoseData& OutAnimationPoseData) const
{
	SCOPE_CYCLE_COUNTER(STAT_BlendSpace_GetAnimPose);
	FScopeCycleCounterUObject BlendSpaceScope(this);

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();

	if (BlendSampleDataCache.Num() == 0)
	{
		ResetToRefPose(OutPose);
		return;
	}

	const bool bNested = InPoseLinks.Num() > 0;
	const int32 NumPoses = BlendSampleDataCache.Num();

	TArray<FCompactPose, TInlineAllocator<8>> ChildrenPoses;
	ChildrenPoses.AddZeroed(NumPoses);

	TArray<FBlendedCurve, TInlineAllocator<8>> ChildrenCurves;
	ChildrenCurves.AddZeroed(NumPoses);

	TArray<UE::Anim::FStackAttributeContainer, TInlineAllocator<8>> ChildrenAttributes;
	ChildrenAttributes.AddZeroed(NumPoses);

	TArray<float, TInlineAllocator<8>> ChildrenWeights;
	ChildrenWeights.AddZeroed(NumPoses);

	for (int32 ChildrenIdx = 0; ChildrenIdx < ChildrenPoses.Num(); ++ChildrenIdx)
	{
		ChildrenPoses[ChildrenIdx].SetBoneContainer(&OutPose.GetBoneContainer());
		ChildrenCurves[ChildrenIdx].InitFrom(OutCurve);
	}

	FAnimExtractContext ChildExtractionContext = ExtractionContext;
	// get all child atoms we interested in
	for (int32 I = 0; I < NumPoses; ++I)
	{
		FCompactPose& Pose = ChildrenPoses[I];

		if (SampleData.IsValidIndex(BlendSampleDataCache[I].SampleDataIndex))
		{
			const FBlendSample& Sample = SampleData[BlendSampleDataCache[I].SampleDataIndex];
			ChildrenWeights[I] = BlendSampleDataCache[I].GetClampedWeight();

			if (bNested)
			{
				check(InPoseLinks.IsValidIndex(BlendSampleDataCache[I].SampleDataIndex));

				// Evaluate the linked graphs
				FPoseContext ChildPoseContext(InProxy, bInExpectsAdditivePose);
				InPoseLinks[BlendSampleDataCache[I].SampleDataIndex].Evaluate(ChildPoseContext);

				// Move out poses etc. for blending
				ChildrenPoses[I] = MoveTemp(ChildPoseContext.Pose);
				ChildrenCurves[I] = MoveTemp(ChildPoseContext.Curve);
				ChildrenAttributes[I] = MoveTemp(ChildPoseContext.CustomAttributes);

				if (Sample.bMirror && MirrorDataTable)
				{
					// @todo: To improve performance, we need to have a mirror map per LOD to avoid computing every time.
					TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex> CompactPoseMirrorBones;
					TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex> ComponentSpaceRefRotations;
					MirrorDataTable->FillCompactPoseAndComponentRefRotations(ChildrenPoses[I].GetBoneContainer(), CompactPoseMirrorBones, ComponentSpaceRefRotations);
					
					FAnimationRuntime::MirrorPose(ChildrenPoses[I], MirrorDataTable->MirrorAxis, CompactPoseMirrorBones, ComponentSpaceRefRotations);
					FAnimationRuntime::MirrorCurves(ChildrenCurves[I], *MirrorDataTable);
					UE::Anim::Attributes::MirrorAttributes(ChildrenAttributes[I], *MirrorDataTable, CompactPoseMirrorBones);
				}
			}
			else
			{
				if (Sample.Animation && Sample.Animation->GetSkeleton() != nullptr)
				{
					const float Time = FMath::Clamp<float>(BlendSampleDataCache[I].Time, 0.f, Sample.Animation->GetPlayLength());

					FAnimationPoseData ChildAnimationPoseData = { Pose, ChildrenCurves[I], ChildrenAttributes[I] };
					// first one always fills up the source one
					ChildExtractionContext.CurrentTime = static_cast<double>(Time);
					ChildExtractionContext.DeltaTimeRecord = BlendSampleDataCache[I].DeltaTimeRecord;
					Sample.Animation->GetAnimationPose(ChildAnimationPoseData, ChildExtractionContext);

					if (Sample.bMirror && MirrorDataTable)
					{
						// @todo: To improve performance, we need to have a mirror map per LOD to avoid computing every time.
						TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex> CompactPoseMirrorBones;
						TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex> ComponentSpaceRefRotations;
						MirrorDataTable->FillCompactPoseAndComponentRefRotations(ChildAnimationPoseData.GetPose().GetBoneContainer(), CompactPoseMirrorBones, ComponentSpaceRefRotations);
					
						FAnimationRuntime::MirrorPose(ChildAnimationPoseData.GetPose(), MirrorDataTable->MirrorAxis, CompactPoseMirrorBones, ComponentSpaceRefRotations);
						FAnimationRuntime::MirrorCurves(ChildAnimationPoseData.GetCurve(), *MirrorDataTable);
						UE::Anim::Attributes::MirrorAttributes(ChildAnimationPoseData.GetAttributes(), *MirrorDataTable, CompactPoseMirrorBones);
					}
				}
				else
				{
					ResetToRefPose(Pose);
				}
			}
		}
		else
		{
			ResetToRefPose(Pose);
		}
	}

	TArrayView<FCompactPose> ChildrenPosesView(ChildrenPoses);

	if (PerBoneBlendValues.Num() > 0)
	{
		bool bValidAdditive = IsValidAdditive();

		if (bAllowMeshSpaceBlending && !bContainsRotationOffsetMeshSpaceSamples)
		{
			// Why blend in mesh space when there are per-bone smoothing settings? Because then we
			// can blend between two aim poses (for example), with the hands moving faster towards
			// the target than the spine. This results in nice organic looking movement, rather than
			// everything moving at the same rate. However, note that if the samples contain mesh-space
			// rotations then the regular blend will already happen in mesh space automatically.
			FAnimationRuntime::BlendPosesTogetherPerBoneInMeshSpace(
				ChildrenPosesView, ChildrenCurves, ChildrenAttributes,
				this, BlendSampleDataCache, OutAnimationPoseData);
		}
		else
		{
			FAnimationRuntime::BlendPosesTogetherPerBone(
				ChildrenPosesView, ChildrenCurves, ChildrenAttributes,
				this, BlendSampleDataCache, OutAnimationPoseData);
		}
	}
	else
	{
		// We could allow mesh space blending here, when there are no per-bone smoothing settings. However, it's
		// unlikely that it would actually provide a benefit, but is a lot more expensive.
		FAnimationRuntime::BlendPosesTogether(
			ChildrenPosesView, ChildrenCurves, ChildrenAttributes, ChildrenWeights, OutAnimationPoseData);
	}

	// Once all the accumulation and blending has been done, normalize rotations.
	OutPose.NormalizeRotations();
}

const FBlendParameter& UBlendSpace::GetBlendParameter(const int32 Index) const
{
	checkf(Index >= 0 && Index < 3, TEXT("Invalid Blend Parameter Index"));
	return BlendParameters[Index];
}

const struct FBlendSample& UBlendSpace::GetBlendSample(const int32 SampleIndex) const
{
#if WITH_EDITOR
	checkf(IsValidBlendSampleIndex(SampleIndex), TEXT("Invalid blend sample index"));
#endif // WITH_EDITOR
	return SampleData[SampleIndex];
}

bool UBlendSpace::GetSamplesFromBlendInput(
	const FVector& BlendInput, TArray<FBlendSampleData>& OutSampleDataList, int32& InOutCachedTriangulationIndex, bool bCombineAnimations) const
{
	if (!bInterpolateUsingGrid)
	{
		OutSampleDataList.Reset(3);

		TArray<FWeightedBlendSample> WeightedBlendSamples;
		FVector NormalizedBlendInput = GetNormalizedBlendInput(BlendInput);
		BlendSpaceData.GetSamples(WeightedBlendSamples, DimensionIndices, NormalizedBlendInput, InOutCachedTriangulationIndex);

		float TotalWeight = 0.0f;
		for (const FWeightedBlendSample& WeightedBlendSample : WeightedBlendSamples)
		{
			float SampleWeight = WeightedBlendSample.SampleWeight;
			if (SampleWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				int32 SampleIndex = WeightedBlendSample.SampleIndex;
				FBlendSampleData BlendSampleData;
				BlendSampleData.SampleDataIndex = SampleIndex;
				BlendSampleData.TotalWeight = SampleWeight;
				BlendSampleData.Animation = SampleData[SampleIndex].Animation;
				BlendSampleData.SamplePlayRate = SampleData[SampleIndex].RateScale;
				OutSampleDataList.Push(BlendSampleData);
				TotalWeight += SampleWeight;
			}
		}
	}
	else
	{
		TArray<FGridBlendSample, TInlineAllocator<4> >& RawGridSamples = FBlendSpaceScratchData::Get().RawGridSamples;
		check(!RawGridSamples.Num()); // this must be called non-recursively
		GetRawSamplesFromBlendInput(BlendInput, RawGridSamples);

		OutSampleDataList.Reset();
		OutSampleDataList.Reserve(RawGridSamples.Num() * FEditorElement::MAX_VERTICES);

		// Consolidate all samples
		for (int32 SampleNum = 0; SampleNum < RawGridSamples.Num(); ++SampleNum)
		{
			FGridBlendSample& GridSample = RawGridSamples[SampleNum];
			float GridWeight = GridSample.BlendWeight;
			FEditorElement& GridElement = GridSample.GridElement;

			for (int32 Ind = 0; Ind < GridElement.MAX_VERTICES; ++Ind)
			{
				const int32 SampleDataIndex = GridElement.Indices[Ind];
				if (SampleData.IsValidIndex(SampleDataIndex))
				{
					int32 Index = OutSampleDataList.AddUnique(SampleDataIndex);
					FBlendSampleData& NewSampleData = OutSampleDataList[Index];

					NewSampleData.AddWeight(GridElement.Weights[Ind] * GridWeight);
					NewSampleData.Animation = SampleData[SampleDataIndex].Animation;
					NewSampleData.SamplePlayRate = SampleData[SampleDataIndex].RateScale;
				}
			}
		}
		RawGridSamples.Reset();
	}

	if (bCombineAnimations)
	{
		// At this point we'll only have one of each sample, but different samples can point to the same
		// animation. We can combine those, making sure to interpolate the parameters like play rate too.
		for (int32 Index1 = 0; Index1 < OutSampleDataList.Num(); ++Index1)
		{
			// Use pointers to make it more obvious what happens if we swap the first and second samples
			FBlendSampleData* FirstSample = &OutSampleDataList[Index1];
			for (int32 Index2 = Index1 + 1; Index2 < OutSampleDataList.Num(); ++Index2)
			{
				FBlendSampleData* SecondSample = &OutSampleDataList[Index2];
				
				const bool bBothUseSingleFrameForBlending = SampleData[FirstSample->SampleDataIndex].bUseSingleFrameForBlending && SampleData[SecondSample->SampleDataIndex].bUseSingleFrameForBlending;
				const bool bFrameIndicesMatch = SampleData[FirstSample->SampleDataIndex].FrameIndexToSample == SampleData[SecondSample->SampleDataIndex].FrameIndexToSample;				
				// if they have same sample, remove the Index2, and get out
				if (FirstSample->SampleDataIndex == SecondSample->SampleDataIndex || // Shouldn't happen
					(FirstSample->Animation != nullptr && FirstSample->Animation == SecondSample->Animation && 
					bBothUseSingleFrameForBlending && bFrameIndicesMatch))
				{
					//Calc New Sample Playrate
					const float TotalWeight = FirstSample->GetClampedWeight() + SecondSample->GetClampedWeight();

					// Only combine playrates if total weight > 0
					if (!FMath::IsNearlyZero(TotalWeight))
					{
						if (FirstSample->GetClampedWeight() < SecondSample->GetClampedWeight())
						{
							// Not strictly necessary, but if we swap here then we keep the one that has a higher
							// weight, which can make debugging/viewing the blend space more intuitive.
							OutSampleDataList.Swap(Index1, Index2);
						}

						const float OriginalWeightedPlayRate = FirstSample->SamplePlayRate * (FirstSample->GetClampedWeight() / TotalWeight);
						const float SecondSampleWeightedPlayRate = SecondSample->SamplePlayRate * (SecondSample->GetClampedWeight() / TotalWeight);
						FirstSample->SamplePlayRate = OriginalWeightedPlayRate + SecondSampleWeightedPlayRate;

						// add weight
						FirstSample->AddWeight(SecondSample->GetClampedWeight());
					}

					// as for time or previous time will be the master one(Index1)
					OutSampleDataList.RemoveAtSwap(Index2, EAllowShrinking::No);
					--Index2;
				}
			}
		}
	}

	OutSampleDataList.Sort([](const FBlendSampleData& A, const FBlendSampleData& B) { return B.TotalWeight < A.TotalWeight; });

	// Remove any below a threshold
	int32 TotalSample = OutSampleDataList.Num();
	float TotalWeight = 0.f;
	for (int32 I = 0; I < TotalSample; ++I)
	{
		if (OutSampleDataList[I].TotalWeight < ZERO_ANIMWEIGHT_THRESH)
		{
			// cut anything in front of this 
			OutSampleDataList.RemoveAt(I, TotalSample - I, EAllowShrinking::No); // we won't shrink here, that might screw up alloc optimization at a higher level, if not this is temp anyway
			break;
		}

		TotalWeight += OutSampleDataList[I].TotalWeight;
	}

	for (int32 I = 0; I < OutSampleDataList.Num(); ++I)
	{
		// normalize to all weights
		OutSampleDataList[I].TotalWeight /= TotalWeight;
	}
	return (OutSampleDataList.Num() != 0);
}

void UBlendSpace::InitializeFilter(FBlendFilter* Filter, int NumDimensions) const
{
	if (Filter)
	{
		Filter->FilterPerAxis.SetNum(NumDimensions);
		for (int32 FilterIndex = 0; FilterIndex != NumDimensions; ++FilterIndex)
		{
			Filter->FilterPerAxis[FilterIndex].Initialize(InterpolationParam[FilterIndex].InterpolationTime,
														  InterpolationParam[FilterIndex].InterpolationType,
														  InterpolationParam[FilterIndex].DampingRatio,
														  BlendParameters[FilterIndex].Min,
														  BlendParameters[FilterIndex].Max,
														  InterpolationParam[FilterIndex].MaxSpeed,
														  !BlendParameters[FilterIndex].bWrapInput);
		}
	}
}

void UBlendSpace::UpdateFilterParams(FBlendFilter* Filter) const
{
	if (Filter)
	{
		for (int32 FilterIndex = 0; FilterIndex != Filter->FilterPerAxis.Num(); ++FilterIndex)
		{
			Filter->FilterPerAxis[FilterIndex].SetParams(InterpolationParam[FilterIndex].DampingRatio,
			                                             BlendParameters[FilterIndex].Min,
			                                             BlendParameters[FilterIndex].Max,
			                                             InterpolationParam[FilterIndex].MaxSpeed,
			                                             !BlendParameters[FilterIndex].bWrapInput);
		}
	}
}

#if WITH_EDITOR
void UBlendSpace::ValidateSampleData()
{
	// (done here since it won't be triggered in the BlendSpace::PostEditChangeProperty, due to empty property during Undo)
	SnapSamplesToClosestGridPoint();

	bool bSampleDataChanged = false;
	AnimLength = 0.f;

	bool bAllMarkerPatternsMatch = true;
	FSyncPattern BlendSpacePattern;

	int32 SampleWithMarkers = INDEX_NONE;

	for (int32 SampleIndex = 0; SampleIndex < SampleData.Num(); ++SampleIndex)
	{
		FBlendSample& Sample = SampleData[SampleIndex];

		// see if same data exists, by same, same values
		for (int32 ComparisonSampleIndex = SampleIndex + 1; ComparisonSampleIndex < SampleData.Num(); ++ComparisonSampleIndex)
		{
			if (IsSameSamplePoint(Sample.SampleValue, SampleData[ComparisonSampleIndex].SampleValue))
			{
				SampleData.RemoveAt(ComparisonSampleIndex);
				--ComparisonSampleIndex;

				bSampleDataChanged = true;
			}
		}

		if (IsAsset())
		{
			const bool bAnimationExists = Sample.Animation != nullptr;
			const bool bSampleInBounds = bAnimationExists ? IsSampleWithinBounds(Sample.SampleValue) : true;
			const bool bSampleIsUnique = bAnimationExists ? !IsTooCloseToExistingSamplePoint(Sample.SampleValue, SampleIndex) : true;
			const bool bShouldForceConditionalPostLoad = bAnimationExists && FUObjectThreadContext::Get().IsRoutingPostLoad && !Sample.Animation->IsDataModelValid();
			if (bShouldForceConditionalPostLoad)
			{
				Sample.Animation->ConditionalPostLoad();
			}
			const bool bSingleFrameBlendingIndexInBounds = IsSingleFrameBlendingIndexInBounds(Sample);
			Sample.bIsValid = bAnimationExists && bSampleInBounds && bSampleIsUnique && bSingleFrameBlendingIndexInBounds;

			if (Sample.bIsValid)
			{
				if (Sample.GetSamplePlayLength() > AnimLength)
				{
					// @todo : should apply scale? If so, we'll need to apply also when blend
					AnimLength = Sample.GetSamplePlayLength();
				}

				Sample.CachedMarkerDataUpdateCounter = Sample.Animation->GetMarkerUpdateCounter();

				if (Sample.Animation->AuthoredSyncMarkers.Num() > 0)
				{
					auto PopulateMarkerNameArray = [](TArray<FName>& Pattern, TArray<struct FAnimSyncMarker>& AuthoredSyncMarkers)
					{
						Pattern.Reserve(AuthoredSyncMarkers.Num());
						for (FAnimSyncMarker& Marker : AuthoredSyncMarkers)
						{
							Pattern.Add(Marker.MarkerName);
						}
					};

					if (SampleWithMarkers == INDEX_NONE && bAllowMarkerBasedSync)
					{
						SampleWithMarkers = SampleIndex;
					}

					if (BlendSpacePattern.MarkerNames.Num() == 0)
					{
						PopulateMarkerNameArray(BlendSpacePattern.MarkerNames, Sample.Animation->AuthoredSyncMarkers);
					}
					else
					{
						TArray<FName> ThisPattern;
						PopulateMarkerNameArray(ThisPattern, Sample.Animation->AuthoredSyncMarkers);
						if (!BlendSpacePattern.DoesPatternMatch(ThisPattern))
						{
							bAllMarkerPatternsMatch = false;
						}
					}
				}
			}
			else
			{
				if (IsRunningGame())
				{
					UE_LOGF(LogAnimation, Error, "[%ls : %d] - Missing Sample Animation", *GetFullName(), SampleIndex + 1);
				}
				else
				{
					static FName NAME_LoadErrors("LoadErrors");
					static const FText HasSampleText = LOCTEXT("EmptyAnimationData_HasSampleShort", "has sample");
					FMessageLog LoadErrors(NAME_LoadErrors);

					TSharedRef<FTokenizedMessage> Message = LoadErrors.Error();
					Message->AddToken(FTextToken::Create(LOCTEXT("EmptyAnimationData_BlendSpace", "BlendSpace")));
					Message->AddToken(FAssetNameToken::Create(GetPathName(), FText::FromString(GetName())));
					if (!bAnimationExists)
					{
						Message->AddToken(FTextToken::Create(LOCTEXT("EmptyAnimationData_HasSample", "has a sample with no/invalid animation. Recommend to remove sample point or set new animation.")));
					}
					else if (!bSampleInBounds)
					{
						Message->AddToken(FTextToken::Create(HasSampleText));
						Message->AddToken(FAssetNameToken::Create(Sample.Animation->GetPathName(), FText::FromString(Sample.Animation->GetName())));
						Message->AddToken(FTextToken::Create(LOCTEXT("EmptyAnimationData_SampleNotInBounds", "that is invalid due to being out of bounds. Recommend adjusting it.")));
					}
					else if (!bSampleIsUnique)
					{
						Message->AddToken(FTextToken::Create(HasSampleText));
						Message->AddToken(FAssetNameToken::Create(Sample.Animation->GetPathName(), FText::FromString(Sample.Animation->GetName())));
						Message->AddToken(FTextToken::Create(LOCTEXT("EmptyAnimationData_NotUnique", "that is not unique. Recommend adjusting it.")));
					}
					else if (!bSingleFrameBlendingIndexInBounds)
					{
						Message->AddToken(FTextToken::Create(HasSampleText));
						Message->AddToken(FAssetNameToken::Create(Sample.Animation->GetPathName(), FText::FromString(Sample.Animation->GetName())));
						Message->AddToken(FTextToken::Create(LOCTEXT("EmptyAnimationData_SingleFrameBlendingNotInBounds", "single frame blending index is out of bounds. Recommend adjusting it.")));
						
						UE_LOGF(LogAnimation, Warning, "[%ls : %d] - single frame blending index: [%d] is out of bounds", *GetFullName(), SampleIndex + 1, Sample.FrameIndexToSample);
					} 
					else
					{
						// Shouldn't get here
						Message->AddToken(FAssetNameToken::Create(Sample.Animation->GetPathName(), FText::FromString(Sample.Animation->GetName())));
						Message->AddToken(FTextToken::Create(LOCTEXT("EmptyAnimationData_IsInvalid", "is invalid.")));
					}
					LoadErrors.Notify();
				}
			}
		}
		else
		{
			Sample.bIsValid = ValidateSampleValue(Sample.SampleValue, SampleIndex);
		}
	}

	bContainsRotationOffsetMeshSpaceSamples = ContainsMatchingSamples(AAT_RotationOffsetMeshSpace);

	SampleIndexWithMarkers = bAllMarkerPatternsMatch ? SampleWithMarkers : INDEX_NONE;

	if (bSampleDataChanged)
	{
		GridSamples.Empty();
		MarkPackageDirty();
	}
}

void UBlendSpace::ExpandRangeForSample(const FVector& SampleValue)
{
	for (int32 Index = 0 ; Index != 3 ; ++Index)
	{
		const float Delta = (BlendParameters[Index].Max - BlendParameters[Index].Min) / BlendParameters[Index].GridNum;
		while (SampleValue[Index] > BlendParameters[Index].Max)
		{
			BlendParameters[Index].Max += Delta;
			BlendParameters[Index].GridNum += 1;
		}
		while (SampleValue[Index] < BlendParameters[Index].Min)
		{
			BlendParameters[Index].Min -= Delta;
			BlendParameters[Index].GridNum += 1;
		}
	}
}

int32 UBlendSpace::AddSample(const FVector& SampleValue)
{
	// We should only be adding samples without a source animation if we are not a standalone asset
	check(!IsAsset());

	// Expand the range if necessary
	ExpandRangeForSample(SampleValue);
	
	const bool bValidSampleData = ValidateSampleValue(SampleValue);

	if (bValidSampleData)
	{
		SampleData.Add(FBlendSample(nullptr, SampleValue, true, bValidSampleData));
		UpdatePreviewBasePose();
	}

	return bValidSampleData ? SampleData.Num() - 1 : INDEX_NONE;
}

int32 UBlendSpace::AddSample(UAnimSequence* AnimationSequence, const FVector& SampleValue)
{
	// Expand the range if necessary
	ExpandRangeForSample(SampleValue);

	const bool bValidSampleData = ValidateSampleValue(SampleValue) && ValidateAnimationSequence(AnimationSequence);

	if (bValidSampleData)
	{
		SampleData.Add(FBlendSample(AnimationSequence, SampleValue, true, bValidSampleData));
		UpdatePreviewBasePose();
	}

	return bValidSampleData ? SampleData.Num() - 1 : INDEX_NONE;
}

bool UBlendSpace::EditSampleValue(const int32 BlendSampleIndex, const FVector& NewValue)
{
	// Expand the range if necessary
	ExpandRangeForSample(NewValue);

	const bool bValidValue = SampleData.IsValidIndex(BlendSampleIndex) && ValidateSampleValue(NewValue, BlendSampleIndex);

	if (bValidValue)
	{
		// Set new value if it passes the tests
		SampleData[BlendSampleIndex].SampleValue = NewValue;
		SampleData[BlendSampleIndex].bIsValid = bValidValue;
	}

	return bValidValue;
}

bool UBlendSpace::ReplaceSampleAnimation(const int32 BlendSampleIndex, UAnimSequence* AnimationSequence)
{
	const bool bValidValue = SampleData.IsValidIndex(BlendSampleIndex);
	if (bValidValue)
	{
		SampleData[BlendSampleIndex].Animation = AnimationSequence;
	}

	return bValidValue;
}

bool UBlendSpace::DeleteSample(const int32 BlendSampleIndex)
{
	const bool bValidRemoval = SampleData.IsValidIndex(BlendSampleIndex);

	if (bValidRemoval)
	{
		SampleData.RemoveAtSwap(BlendSampleIndex);
		UpdatePreviewBasePose();
	}

	return bValidRemoval;
}

bool UBlendSpace::IsValidBlendSampleIndex(const int32 SampleIndex) const
{
	return SampleData.IsValidIndex(SampleIndex);
}

bool UBlendSpace::IsSingleFrameBlendingIndexInBounds(const FBlendSample& BlendSample)
{
	if (BlendSample.bUseSingleFrameForBlending)
	{
		if (BlendSample.Animation && BlendSample.Animation->IsDataModelValid())
		{
			const uint32 NumberOfFrames = static_cast<uint32>(BlendSample.Animation->GetDataModelInterface()->GetNumberOfFrames());		
			if (BlendSample.FrameIndexToSample >= 0 && BlendSample.FrameIndexToSample <= NumberOfFrames)
			{
				return true;
			}
		}

		// either we are not using single frame blending or index is outside of bounds (or data model is invalid, in which case can't report this error properly)
		return false;
	}
	
	return true;
}

const TArray<FEditorElement>& UBlendSpace::GetGridSamples() const
{
	return GridSamples;
}

const FBlendSpaceData& UBlendSpace::GetBlendSpaceData() const
{
	return BlendSpaceData;
}

void UBlendSpace::FillupGridElements(const TArray<FEditorElement>& GridElements, const TArray<int32>& InDimensionIndices)
{
	DimensionIndices = InDimensionIndices;

	GridSamples.Empty(GridElements.Num());
	GridSamples.AddUninitialized(GridElements.Num());

	for (int32 ElementIndex = 0; ElementIndex < GridElements.Num(); ++ElementIndex)
	{
		const FEditorElement& ViewGrid = GridElements[ElementIndex];
		FEditorElement NewGrid;
		float TotalWeight = 0.f;
		for (int32 VertexIndex = 0; VertexIndex < FEditorElement::MAX_VERTICES; ++VertexIndex)
		{
			const int32 SampleIndex = ViewGrid.Indices[VertexIndex];
			if (SampleIndex != INDEX_NONE)
			{
				NewGrid.Indices[VertexIndex] = SampleIndex;
			}
			else
			{
				NewGrid.Indices[VertexIndex] = INDEX_NONE;
			}

			if (NewGrid.Indices[VertexIndex] == INDEX_NONE)
			{
				NewGrid.Weights[VertexIndex] = 0.f;
			}
			else
			{
				NewGrid.Weights[VertexIndex] = ViewGrid.Weights[VertexIndex];
				TotalWeight += ViewGrid.Weights[VertexIndex];
			}
		}

		// Need to normalize the weights
		if (TotalWeight > 0.f)
		{
			for (int32 J = 0; J < FEditorElement::MAX_VERTICES; ++J)
			{
				NewGrid.Weights[J] /= TotalWeight;
			}
		}

		GridSamples[ElementIndex] = NewGrid;
	}
}

void UBlendSpace::EmptyGridElements()
{
	GridSamples.Empty();
}

bool UBlendSpace::ValidateAnimationSequence(const UAnimSequence* AnimationSequence) const
{
	const bool bValidAnimationSequence = IsAnimationCompatible(AnimationSequence)
		&& IsAnimationCompatibleWithSkeleton(AnimationSequence)
		&& (GetNumberOfBlendSamples() == 0 || DoesAnimationMatchExistingSamples(AnimationSequence));

	return bValidAnimationSequence;
}

bool UBlendSpace::DoesAnimationMatchExistingSamples(const UAnimSequence* AnimationSequence) const
{
	const bool bMatchesExistingAnimations = ContainsMatchingSamples(AnimationSequence->AdditiveAnimType);
	return bMatchesExistingAnimations;
}

bool UBlendSpace::ShouldAnimationBeAdditive() const
{
	const bool bShouldBeAdditive = !ContainsNonAdditiveSamples();
	return bShouldBeAdditive;
}

bool UBlendSpace::IsAnimationCompatibleWithSkeleton(const UAnimSequence* AnimationSequence) const
{
	// Check if the animation sequences skeleton is compatible with the blendspace one
	const USkeleton* MySkeleton = GetSkeleton();
	bool bIsAnimationCompatible = AnimationSequence && MySkeleton && AnimationSequence->GetSkeleton();
#if WITH_EDITORONLY_DATA
	bIsAnimationCompatible = bIsAnimationCompatible && MySkeleton->IsCompatibleForEditor(AnimationSequence->GetSkeleton());
#endif
	return bIsAnimationCompatible;
}

bool UBlendSpace::IsAnimationCompatible(const UAnimSequence* AnimationSequence) const
{
	// If the supplied animation is of a different additive animation type or this blendspace support non-additive animations
	const bool bIsCompatible = IsValidAdditiveType(AnimationSequence->AdditiveAnimType);
	return bIsCompatible;
}

bool UBlendSpace::ValidateSampleValue(const FVector& SampleValue, int32 OriginalIndex) const
{
	bool bValid = true;
	bValid &= IsSampleWithinBounds(SampleValue);
	bValid &= !IsTooCloseToExistingSamplePoint(SampleValue, OriginalIndex);
	bValid &= OriginalIndex == INDEX_NONE || IsSingleFrameBlendingIndexInBounds(SampleData[OriginalIndex]);
	return bValid;
}

bool UBlendSpace::IsSampleWithinBounds(const FVector& SampleValue) const
{
	return !((SampleValue.X < BlendParameters[0].Min) ||
		(SampleValue.X > BlendParameters[0].Max) ||
		(SampleValue.Y < BlendParameters[1].Min) ||
		(SampleValue.Y > BlendParameters[1].Max));
}

bool UBlendSpace::IsTooCloseToExistingSamplePoint(const FVector& SampleValue, int32 OriginalIndex) const
{
	bool bMatchesSamplePoint = false;
	for (int32 SampleIndex = 0; SampleIndex < SampleData.Num(); ++SampleIndex)
	{
		if (SampleIndex != OriginalIndex)
		{
			if (IsSameSamplePoint(SampleValue, SampleData[SampleIndex].SampleValue))
			{
				bMatchesSamplePoint = true;
				break;
			}
		}
	}

	return bMatchesSamplePoint;
}

#endif // WITH_EDITOR

// When using CriticallyDampedSmoothing, how to go from the interpolation speed to the smooth
// time? What would the critically damped velocity be as it goes from a starting value of 0 to a
// target of 1 (see eq in CriticallyDampedSmoothing), starting with v = 0?
//
// v = W^2 t exp(-W t)
//
// Differentiate and set equal to zero to find maximum v is at t = 1 / W
//
// vMax = W / e = 2 / (SmoothingTime * e)
//
// Set this equal to TargetWeightInterpolationSpeedPerSec, we get
//
// SmoothingTime = 2 / (e * TargetWeightInterpolationSpeedPerSec)
//
// However - this looks significantly slower than when using a constant speed, because we're
// easing in/out, so aim for twice this speed (i.e. smooth over half the time)
static float SmoothingTimeFromSpeed(float Speed)
{
	return Speed > FLT_EPSILON ? 1.0f / (UE_EULERS_NUMBER * Speed) : 0.0f;
}

static float SpeedFromSmoothingTime(float SmoothingTime)
{
	return SmoothingTime > FLT_EPSILON ? 1 / (UE_EULERS_NUMBER * SmoothingTime) : 0.0f;
}

void UBlendSpace::InitializePerBoneBlend()
{
	if (PerBoneBlendMode == EBlendSpacePerBoneBlendMode::ManualPerBoneOverride)
	{
		PerBoneBlendValues = ManualPerBoneOverrides;
	}
	else
	{
		PerBoneBlendValues.Empty();

		const UBlendProfile* BlendProfile = PerBoneBlendProfile.BlendProfile.Get();
		if (BlendProfile)
		{
			const int32 NumBlendEntries = BlendProfile->GetNumBlendEntries();
			for (int32 EntryIndex = 0; EntryIndex < NumBlendEntries; ++EntryIndex)
			{
				const FBlendProfileBoneEntry& BoneEntry = BlendProfile->GetEntry(EntryIndex);

				FPerBoneInterpolation BoneInterpolation;
				BoneInterpolation.BoneReference = BoneEntry.BoneReference;

				const float TargetWeightInterpolationTime = SmoothingTimeFromSpeed(TargetWeightInterpolationSpeedPerSec);
				const float BlendProfileInterpolationTime = SmoothingTimeFromSpeed(PerBoneBlendProfile.TargetWeightInterpolationSpeedPerSec);

				const float InterpolatedTime = FMath::Lerp(TargetWeightInterpolationTime, BlendProfileInterpolationTime, FMath::Clamp(BoneEntry.BlendScale, 0.0f, 1.0f));
				BoneInterpolation.InterpolationSpeedPerSec = SpeedFromSmoothingTime(InterpolatedTime);

				PerBoneBlendValues.Add(BoneInterpolation);
			}
		}
	}

	const USkeleton* MySkeleton = GetSkeleton();
	for (FPerBoneInterpolation& BoneInterpolationData : PerBoneBlendValues)
	{
		BoneInterpolationData.Initialize(MySkeleton);
	}
}

void UBlendSpace::TickFollowerSamples(
	TArray<FBlendSampleData>& SampleDataList, const int32 HighestWeightIndex, FAnimAssetTickContext& Context, 
	bool bResetMarkerDataOnFollowers, bool bLooping, const UMirrorDataTable* InMirrorDataTable) const
{
	FAnimTickRecord TempTickRecord;
	TempTickRecord.MirrorDataTable = InMirrorDataTable;
	TickFollowerSamples(SampleDataList, HighestWeightIndex, Context, bResetMarkerDataOnFollowers, bLooping, TempTickRecord);
}

void UBlendSpace::TickFollowerSamples(TArray<FBlendSampleData>& SampleDataList, const int32 HighestWeightIndex, FAnimAssetTickContext& Context,
	bool bResetMarkerDataOnFollowers, bool bLooping, const FAnimTickRecord& BlendSpaceInstance) const
{
	for (int32 SampleIndex = 0; SampleIndex < SampleDataList.Num(); ++SampleIndex)
	{
		FBlendSampleData& SampleDataItem = SampleDataList[SampleIndex];
		const FBlendSample& Sample = SampleData[SampleDataItem.SampleDataIndex];

		if (HighestWeightIndex != SampleIndex && Sample.Animation != nullptr)
		{
			if (bResetMarkerDataOnFollowers)
			{
				SampleDataItem.MarkerTickRecord.Reset();
			}

			// Update followers who can do marker sync, others will be handled later in TickAssetPlayer
			if (Sample.Animation->AuthoredSyncMarkers.Num() > 0) 
			{
				Sample.Animation->TickByMarkerAsFollower(
					SampleDataItem.MarkerTickRecord, Context.MarkerTickContext, SampleDataItem.Time, 
					SampleDataItem.PreviousTime, Context.GetLeaderDelta(), bLooping, GetEffectiveMirrorDataTableForSample(BlendSpaceInstance, Sample));
			}
		}
	}
}

float UBlendSpace::GetAnimationLengthFromSampleData(const TArray<FBlendSampleData>& SampleDataList) const
{
	if (bUseLegacySamplePointAnimationLengthCalculations)
	{
		float BlendAnimLength = 0.f;
		for (int32 I = 0; I < SampleDataList.Num(); ++I)
		{
			const int32 SampleDataIndex = SampleDataList[I].SampleDataIndex;
			if (SampleData.IsValidIndex(SampleDataIndex))
			{
				const FBlendSample& Sample = SampleData[SampleDataIndex];
				if (Sample.Animation)
				{
					//Use the SamplePlayRate from the SampleDataList, not the RateScale from SampleData as SamplePlayRate might contain
					//Multiple samples contribution which we would otherwise lose
					const float MultipliedSampleRateScale = Sample.Animation->RateScale * SampleDataList[I].SamplePlayRate;
					// apply rate scale to get actual playback time
					BlendAnimLength += (Sample.GetSamplePlayLength() / ((MultipliedSampleRateScale) != 0.0f ? FMath::Abs(MultipliedSampleRateScale) : 1.0f)) * SampleDataList[I].GetClampedWeight();

					UE_LOGF(LogAnimation, Verbose, "[%d] - Sample Animation(%ls) : Weight(%0.5f) ", I + 1, *Sample.Animation->GetName(), SampleDataList[I].GetClampedWeight());
				}
			}
		}

		return BlendAnimLength;
	}
	else
	{
		float BlendedNormalizedSpeed = 0.0f;

		for (int32 I = 0; I < SampleDataList.Num(); ++I)
		{
			const int32 SampleDataIndex = SampleDataList[I].SampleDataIndex;
			if (SampleData.IsValidIndex(SampleDataIndex))
			{
				const FBlendSample& Sample = SampleData[SampleDataIndex];
				if (Sample.Animation)
				{
					const float SamplePlayRate = Sample.Animation->RateScale * SampleDataList[I].SamplePlayRate;
					const float SampleAnimLength = Sample.GetSamplePlayLength();
					const float SampleWeight = SampleDataList[I].GetClampedWeight();

					const float SampleNormalizedSpeed = (SampleAnimLength != 0.0f) ? (SamplePlayRate / SampleAnimLength) : 0.0f;
					BlendedNormalizedSpeed += SampleNormalizedSpeed * SampleWeight;

					UE_LOGF(LogAnimation, Verbose, "[%d] - Sample Animation(%ls) : Weight(%0.5f) ", I + 1, *Sample.Animation->GetName(), SampleDataList[I].GetClampedWeight());
				}
			}
		}

		if (BlendedNormalizedSpeed == 0.0f)
		{
			return 0.0f;
		}
		else
		{
			const float BlendedAnimLength = 1.0f / BlendedNormalizedSpeed;
			return BlendedAnimLength;
		}
	}
}

FVector UBlendSpace::GetClampedBlendInput(const FVector& BlendInput)  const
{
	FVector AdjustedInput = BlendInput;
	for (int iAxis = 0; iAxis != 3; ++iAxis)
	{
		if (!BlendParameters[iAxis].bWrapInput)
		{
			AdjustedInput[iAxis] = FMath::Clamp(AdjustedInput[iAxis], BlendParameters[iAxis].Min, BlendParameters[iAxis].Max);
		}
	}
	return AdjustedInput;
}

FVector UBlendSpace::GetClampedAndWrappedBlendInput(const FVector& BlendInput) const
{
	FVector AdjustedInput = BlendInput;
	for (int iAxis = 0; iAxis != 3; ++iAxis)
	{
		if (BlendParameters[iAxis].bWrapInput)
		{
			AdjustedInput[iAxis] = FMath::Wrap<FVector::FReal>(AdjustedInput[iAxis], BlendParameters[iAxis].Min, BlendParameters[iAxis].Max);
		}
		else
		{
			AdjustedInput[iAxis] = FMath::Clamp<FVector::FReal>(AdjustedInput[iAxis], BlendParameters[iAxis].Min, BlendParameters[iAxis].Max);
		}
	}
	return AdjustedInput;
}

FVector UBlendSpace::ConvertBlendInputToGridSpace(const FVector& BlendInput) const
{
	FVector AdjustedInput = GetClampedAndWrappedBlendInput(BlendInput);

	const FVector MinBlendInput = FVector(BlendParameters[0].Min, BlendParameters[1].Min, BlendParameters[2].Min);
	const FVector GridSize = FVector(BlendParameters[0].GetGridSize(), BlendParameters[1].GetGridSize(), BlendParameters[2].GetGridSize());

	FVector NormalizedBlendInput = (AdjustedInput - MinBlendInput) / GridSize;
	return NormalizedBlendInput;
}

FVector UBlendSpace::GetNormalizedBlendInput(const FVector& BlendInput) const
{
	FVector AdjustedInput = GetClampedAndWrappedBlendInput(BlendInput);

	const FVector MinBlendInput = FVector(BlendParameters[0].Min, BlendParameters[1].Min, BlendParameters[2].Min);
	const FVector MaxBlendInput = FVector(BlendParameters[0].Max, BlendParameters[1].Max, BlendParameters[2].Max);

	FVector NormalizedBlendInput = (AdjustedInput - MinBlendInput) / (MaxBlendInput - MinBlendInput);
	return NormalizedBlendInput;
}

const FEditorElement* UBlendSpace::GetGridSampleInternal(int32 Index) const
{
	return GridSamples.IsValidIndex(Index) ? &GridSamples[Index] : nullptr;
}

static void SmoothWeight(float& Output, float& OutputRate, float Input, float InputRate, float Target, float DeltaTime, float Speed, bool bUseEaseInOut)
{
	if (Speed <= 0.0f)
	{
		Output = Target;
		return;
	}

	if (bUseEaseInOut)
	{
		Output = Input;
		OutputRate = InputRate;
		FMath::CriticallyDampedSmoothing(Output, OutputRate, Target, 0.0f, DeltaTime, SmoothingTimeFromSpeed(Speed));
	}
	else
	{
		Output = FMath::FInterpConstantTo(Input, Target, DeltaTime, Speed);
	}
}

bool UBlendSpace::InterpolateWeightOfSampleData(float DeltaTime, const TArray<FBlendSampleData>& OldSampleDataList, const TArray<FBlendSampleData>& NewSampleDataList, TArray<FBlendSampleData>& FinalSampleDataList) const
{
	float TotalFinalWeight = 0.f;
	float TotalFinalPerBoneWeight = 0.0f;

	// now interpolate from old to new target, this is brute-force
	for (auto OldIt = OldSampleDataList.CreateConstIterator(); OldIt; ++OldIt)
	{
		// Now need to modify old sample, so copy it
		FBlendSampleData OldSample = *OldIt;
		bool bTargetSampleExists = false;

		if (OldSample.PerBoneBlendData.Num() != PerBoneBlendValues.Num())
		{
			OldSample.PerBoneBlendData.Init(OldSample.TotalWeight, PerBoneBlendValues.Num());
			OldSample.PerBoneWeightRate.Init(OldSample.WeightRate, PerBoneBlendValues.Num());
		}

		for (auto NewIt = NewSampleDataList.CreateConstIterator(); NewIt; ++NewIt)
		{
			const FBlendSampleData& NewSample = *NewIt;
			// if same sample is found, interpolate
			if (NewSample.SampleDataIndex == OldSample.SampleDataIndex)
			{
				FBlendSampleData InterpData = NewSample;
				SmoothWeight(InterpData.TotalWeight, InterpData.WeightRate, OldSample.TotalWeight, OldSample.WeightRate, NewSample.TotalWeight, DeltaTime, TargetWeightInterpolationSpeedPerSec, bTargetWeightInterpolationEaseInOut);
				InterpData.PerBoneBlendData = OldSample.PerBoneBlendData;
				InterpData.PerBoneWeightRate = OldSample.PerBoneWeightRate;

				// now interpolate the per bone weights
				float TotalPerBoneWeight = 0.0f;
				for (int32 Iter = 0; Iter < InterpData.PerBoneBlendData.Num(); ++Iter)
				{
					SmoothWeight(
						InterpData.PerBoneBlendData[Iter], InterpData.PerBoneWeightRate[Iter],
						OldSample.PerBoneBlendData[Iter], OldSample.PerBoneWeightRate[Iter], NewSample.TotalWeight,
						DeltaTime, PerBoneBlendValues[Iter].InterpolationSpeedPerSec, bTargetWeightInterpolationEaseInOut);
					TotalPerBoneWeight += InterpData.PerBoneBlendData[Iter];
				}

				if (InterpData.TotalWeight > ZERO_ANIMWEIGHT_THRESH || TotalPerBoneWeight > ZERO_ANIMWEIGHT_THRESH)
				{
					FinalSampleDataList.Add(InterpData);
					TotalFinalWeight += InterpData.GetClampedWeight();
					TotalFinalPerBoneWeight += TotalPerBoneWeight;
					bTargetSampleExists = true;
					break;
				}
			}
		}

		// if new target isn't found, interpolate to 0.f, this is gone
		if (bTargetSampleExists == false)
		{
			FBlendSampleData InterpData = OldSample;
			SmoothWeight(InterpData.TotalWeight, InterpData.WeightRate, OldSample.TotalWeight, OldSample.WeightRate, 0.0f, DeltaTime, TargetWeightInterpolationSpeedPerSec, bTargetWeightInterpolationEaseInOut);
			// now interpolate the per bone weights
			float TotalPerBoneWeight = 0.0f;
			for (int32 Iter = 0; Iter < InterpData.PerBoneBlendData.Num(); ++Iter)
			{
				SmoothWeight(
					InterpData.PerBoneBlendData[Iter], InterpData.PerBoneWeightRate[Iter],
					OldSample.PerBoneBlendData[Iter], OldSample.PerBoneWeightRate[Iter], 0.0f,
					DeltaTime, PerBoneBlendValues[Iter].InterpolationSpeedPerSec, bTargetWeightInterpolationEaseInOut);
				TotalPerBoneWeight += InterpData.PerBoneBlendData[Iter];
			}

			// add it if it's not zero
			if (InterpData.TotalWeight > ZERO_ANIMWEIGHT_THRESH || TotalPerBoneWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				FinalSampleDataList.Add(InterpData);
				TotalFinalWeight += InterpData.GetClampedWeight();
				TotalFinalPerBoneWeight += TotalPerBoneWeight;
			}
		}
	}

	// now find new samples that are not found from old samples
	for (auto OldIt = NewSampleDataList.CreateConstIterator(); OldIt; ++OldIt)
	{
		// Now need to modify old sample, so copy it
		FBlendSampleData OldSample = *OldIt;
		bool bOldSampleExists = false;

		if (OldSample.PerBoneBlendData.Num() != PerBoneBlendValues.Num())
		{
			OldSample.PerBoneBlendData.Init(OldSample.TotalWeight, PerBoneBlendValues.Num());
			OldSample.PerBoneWeightRate.Init(OldSample.WeightRate, PerBoneBlendValues.Num());
		}

		for (auto NewIt = FinalSampleDataList.CreateConstIterator(); NewIt; ++NewIt)
		{
			const FBlendSampleData& NewSample = *NewIt;
			if (NewSample.SampleDataIndex == OldSample.SampleDataIndex)
			{
				bOldSampleExists = true;
				break;
			}
		}

		// add those new samples
		if (bOldSampleExists == false)
		{
			FBlendSampleData InterpData = OldSample;
			float TargetWeight = InterpData.TotalWeight;
			OldSample.TotalWeight = 0.0f;
			OldSample.WeightRate = 0.0f;
			SmoothWeight(InterpData.TotalWeight, InterpData.WeightRate, OldSample.TotalWeight, OldSample.WeightRate, TargetWeight, DeltaTime, TargetWeightInterpolationSpeedPerSec, bTargetWeightInterpolationEaseInOut);
			// now interpolate the per bone weights
			float TotalPerBoneWeight = 0.0f;
			for (int32 Iter = 0; Iter < InterpData.PerBoneBlendData.Num(); ++Iter)
			{
				float Target = OldSample.PerBoneBlendData[Iter];
				OldSample.PerBoneBlendData[Iter] = 0.0f;
				OldSample.PerBoneWeightRate[Iter] = 0.0f;
				SmoothWeight(
					InterpData.PerBoneBlendData[Iter], InterpData.PerBoneWeightRate[Iter],
					OldSample.PerBoneBlendData[Iter], OldSample.PerBoneWeightRate[Iter], Target,
					DeltaTime, PerBoneBlendValues[Iter].InterpolationSpeedPerSec, bTargetWeightInterpolationEaseInOut);
				TotalPerBoneWeight += InterpData.PerBoneBlendData[Iter];
			}
			if (InterpData.TotalWeight > ZERO_ANIMWEIGHT_THRESH || TotalPerBoneWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				FinalSampleDataList.Add(InterpData);
				TotalFinalWeight += InterpData.GetClampedWeight();
				TotalFinalPerBoneWeight += TotalPerBoneWeight;
			}
		}
	}

	return TotalFinalWeight > ZERO_ANIMWEIGHT_THRESH || TotalFinalPerBoneWeight > ZERO_ANIMWEIGHT_THRESH;
}

FVector UBlendSpace::FilterInput(FBlendFilter* Filter, const FVector& BlendInput, float DeltaTime) const
{
	FVector FilteredBlendInput = BlendInput;
	if(Filter)
	{
#if WITH_EDITOR
		if (Filter->FilterPerAxis.IsEmpty())
		{
			InitializeFilter(Filter);
		}
		else
		{
			for (int32 AxisIndex = 0; AxisIndex < Filter->FilterPerAxis.Num(); ++AxisIndex)
			{
				if (Filter->FilterPerAxis[AxisIndex].NeedsUpdate(InterpolationParam[AxisIndex].InterpolationType,
																 InterpolationParam[AxisIndex].InterpolationTime))
				{
					InitializeFilter(Filter);
					break;
				}
			}
		}
		// Note that if we expose the damping ratio etc as pins, this should be called outside of the editor too.
		UpdateFilterParams(Filter);
#endif
		for (int32 AxisIndex = 0; AxisIndex < Filter->FilterPerAxis.Num(); ++AxisIndex)
		{
			if (BlendParameters[AxisIndex].bWrapInput)
			{
				Filter->FilterPerAxis[AxisIndex].WrapToValue(
					BlendInput[AxisIndex], BlendParameters[AxisIndex].Max - BlendParameters[AxisIndex].Min);
			}
			FilteredBlendInput[AxisIndex] = Filter->FilterPerAxis[AxisIndex].UpdateAndGetFilteredData(
				BlendInput[AxisIndex], DeltaTime);
		}
	}
	return FilteredBlendInput;
}

bool UBlendSpace::ContainsMatchingSamples(EAdditiveAnimationType AdditiveType) const
{
	bool bMatching = true;
	for (const FBlendSample& Sample : SampleData)
	{
		const UAnimSequence* Animation = Sample.Animation;
		bMatching &= (SampleData.Num() > 1 && Animation == nullptr) || (Animation && ((AdditiveType == AAT_None) ? true : Animation->IsValidAdditive()) && Animation->AdditiveAnimType == AdditiveType);

		if (bMatching == false)
		{
			break;
		}
	}

	return bMatching && SampleData.Num() > 0;
}

bool UBlendSpace::IsSameSamplePoint(const FVector& SamplePointA, const FVector& SamplePointB) const
{
#if 1
	return FMath::IsNearlyEqual(SamplePointA.X, SamplePointB.X, (FVector::FReal)UE_KINDA_SMALL_NUMBER)
		&& FMath::IsNearlyEqual(SamplePointA.Y, SamplePointB.Y, (FVector::FReal)UE_KINDA_SMALL_NUMBER)
		&& FMath::IsNearlyEqual(SamplePointA.Z, SamplePointB.Z, (FVector::FReal)UE_KINDA_SMALL_NUMBER);
#else
	if (DimensionIndices.Num() == 0 || DimensionIndices.Num() > 3)
	{
		UE_LOGF(LogAnimation, Warning, "Unhandled dimensionality in samples: %d", DimensionIndices.Num());
		return false;
	}
	for (int32 iDim = 0; iDim != DimensionIndices.Num(); ++iDim)
	{
		if (!FMath::IsNearlyEqual(SamplePointA[iDim], SamplePointB[iDim]))
			return false;
	}
	return true;
#endif
}


#if WITH_EDITOR
bool UBlendSpace::ContainsNonAdditiveSamples() const
{
	return ContainsMatchingSamples(AAT_None);
}

void UBlendSpace::UpdatePreviewBasePose()
{
#if WITH_EDITORONLY_DATA
	PreviewBasePose = nullptr;
	// Check if blendspace is additive and try to find a ref pose 
	if (IsValidAdditive())
	{
		for (const FBlendSample& BlendSample : SampleData)
		{
			if (BlendSample.Animation && BlendSample.Animation->RefPoseSeq)
			{
				PreviewBasePose = BlendSample.Animation->RefPoseSeq;
				break;
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UBlendSpace::UpdateBlendSpacesUsingAnimSequence(UAnimSequenceBase* Sequence)
{
	for (TObjectIterator<UBlendSpace> BlendSpaceIt; BlendSpaceIt; ++BlendSpaceIt)
	{
		TArray<UAnimationAsset*> ReferredAssets;
		BlendSpaceIt->GetAllAnimationSequencesReferred(ReferredAssets, false);

		if (ReferredAssets.Contains(Sequence))
		{
			BlendSpaceIt->Modify();
			BlendSpaceIt->ValidateSampleData();
		}
	}
}

#endif // WITH_EDITOR

TArray<FName>* UBlendSpace::GetUniqueMarkerNames()
{
	if (SampleIndexWithMarkers != INDEX_NONE && SampleData.Num() > SampleIndexWithMarkers)
	{
		FBlendSample& BlendSample = SampleData[SampleIndexWithMarkers];
		if (BlendSample.Animation != nullptr)
		{
			return BlendSample.Animation->GetUniqueMarkerNames();
		}
	}

	return nullptr;
}

void UBlendSpace::GetRawSamplesFromBlendInput(const FVector& BlendInput, TArray<FGridBlendSample, TInlineAllocator<4> >& OutBlendSamples) const
{
	switch (DimensionIndices.Num())
	{
	case 1:
		GetRawSamplesFromBlendInput1D(BlendInput, OutBlendSamples);
		break;
	case 2:
		GetRawSamplesFromBlendInput2D(BlendInput, OutBlendSamples);
		break;
	default:
		UE_LOGF(LogAnimation, Warning, "Unhandled dimensionality in samples: %d", DimensionIndices.Num());
		break;
	}
}

/*-----------------------------------------------------------------------------
	1D functionality.
-----------------------------------------------------------------------------*/

void UBlendSpace::GetRawSamplesFromBlendInput1D(const FVector& BlendInput, TArray<FGridBlendSample, TInlineAllocator<4> >& OutBlendSamples) const
{
	check(DimensionIndices.Num() == 1);
	int32 Index0 = DimensionIndices[0];

	FVector NormalizedBlendInput = ConvertBlendInputToGridSpace(BlendInput);

	float GridIndex = FMath::TruncToFloat(NormalizedBlendInput[Index0]);
	float Remainder = NormalizedBlendInput[Index0] - GridIndex;

	const FEditorElement* BeforeElement = GetGridSampleInternal(GridIndex);

	if (BeforeElement)
	{
		FGridBlendSample NewSample;
		NewSample.GridElement = *BeforeElement;
		// now calculate weight - GridElement has weights to nearest samples, here we weight the grid element
		NewSample.BlendWeight = (1.f - Remainder);
		OutBlendSamples.Add(NewSample);
	}
	else
	{
		FGridBlendSample NewSample;
		NewSample.GridElement = FEditorElement();
		NewSample.BlendWeight = 0.f;
		OutBlendSamples.Add(NewSample);
	}

	const FEditorElement* AfterElement = GetGridSampleInternal(GridIndex + 1);

	if (AfterElement)
	{
		FGridBlendSample NewSample;
		NewSample.GridElement = *AfterElement;
		// now calculate weight - GridElement has weights to nearest samples, here we weight the grid element
		NewSample.BlendWeight = (Remainder);
		OutBlendSamples.Add(NewSample);
	}
	else
	{
		FGridBlendSample NewSample;
		NewSample.GridElement = FEditorElement();
		NewSample.BlendWeight = 0.f;
		OutBlendSamples.Add(NewSample);
	}
}


/*-----------------------------------------------------------------------------
	2D functionality.
-----------------------------------------------------------------------------*/

const FEditorElement* UBlendSpace::GetEditorElement(int32 XIndex, int32 YIndex) const
{
	// Needs to match FBlendSpaceGrid::GetElement and FBlendSpaceGrid::GetElement
	int32 Index = YIndex * (BlendParameters[0].GridNum + 1) + XIndex;
	return GetGridSampleInternal(Index);
}

void UBlendSpace::GetRawSamplesFromBlendInput2D(const FVector& BlendInput, TArray<FGridBlendSample, TInlineAllocator<4> >& OutBlendSamples) const
{
	check(DimensionIndices.Num() == 2);

	OutBlendSamples.Reset();
	OutBlendSamples.AddUninitialized(4);

	FGridBlendSample& LeftBottom = OutBlendSamples[0];
	FGridBlendSample& RightBottom = OutBlendSamples[1];
	FGridBlendSample& LeftTop = OutBlendSamples[2];
	FGridBlendSample& RightTop = OutBlendSamples[3];

	const FVector NormalizedBlendInput = ConvertBlendInputToGridSpace(BlendInput);
	const FVector GridIndex(FMath::TruncToFloat(NormalizedBlendInput.X), FMath::TruncToFloat(NormalizedBlendInput.Y), 0.f);
	const FVector Remainder = NormalizedBlendInput - GridIndex;
	// bi-linear very simple interpolation
	const FEditorElement* EleLT = GetEditorElement(GridIndex.X, GridIndex.Y + 1);
	if (EleLT)
	{
		LeftTop.GridElement = *EleLT;
		// now calculate weight - distance to each corner since input is already normalized within grid, we can just calculate distance 
		LeftTop.BlendWeight = (1.f - Remainder.X) * Remainder.Y;
	}
	else
	{
		LeftTop.GridElement = FEditorElement();
		LeftTop.BlendWeight = 0.f;
	}

	const FEditorElement* EleRT = GetEditorElement(GridIndex.X + 1, GridIndex.Y + 1);
	if (EleRT)
	{
		RightTop.GridElement = *EleRT;
		RightTop.BlendWeight = Remainder.X * Remainder.Y;
	}
	else
	{
		RightTop.GridElement = FEditorElement();
		RightTop.BlendWeight = 0.f;
	}

	const FEditorElement* EleLB = GetEditorElement(GridIndex.X, GridIndex.Y);
	if (EleLB)
	{
		LeftBottom.GridElement = *EleLB;
		LeftBottom.BlendWeight = (1.f - Remainder.X) * (1.f - Remainder.Y);
	}
	else
	{
		LeftBottom.GridElement = FEditorElement();
		LeftBottom.BlendWeight = 0.f;
	}

	const FEditorElement* EleRB = GetEditorElement(GridIndex.X + 1, GridIndex.Y);
	if (EleRB)
	{
		RightBottom.GridElement = *EleRB;
		RightBottom.BlendWeight = Remainder.X * (1.f - Remainder.Y);
	}
	else
	{
		RightBottom.GridElement = FEditorElement();
		RightBottom.BlendWeight = 0.f;
	}
}


#if WITH_EDITOR

//======================================================================================================================
FVector UBlendSpace::GetGridPosition(int32 GridX, int32 GridY) const
{
	const FVector GridMin(BlendParameters[0].Min, BlendParameters[1].Min, 0.0f);
	const FVector GridMax(BlendParameters[0].Max, BlendParameters[1].Max, 0.0f);
	const FVector GridRange(GridMax.X - GridMin.X, GridMax.Y - GridMin.Y, 0.0f);
	const FIntPoint NumGridDivisions(BlendParameters[0].GridNum, BlendParameters[1].GridNum);

	const FVector GridPoint(
		GridMin.X + (GridX * GridRange.X / NumGridDivisions.X), 
		GridMin.Y + (GridY * GridRange.Y / NumGridDivisions.Y),
		0.0f);
	return GridPoint;
}

//======================================================================================================================
FVector UBlendSpace::GetGridPosition(int32 GridIndex) const
{
	// Needs to match FBlendSpaceGrid::GetElement and UBlendSpace::GetEditorElement
	int32 GridX = GridIndex % (BlendParameters[0].GridNum + 1);
	int32 GridY = (GridIndex - GridX) / (BlendParameters[0].GridNum + 1);
	return GetGridPosition(GridX, GridY);
}

//======================================================================================================================
// Note that this runs and is needed if the axes settings are change to enable snapping - then we do
// a pass over all the samples and make sure everything is in order. When samples are being dragged
// around/dropped then that moving sample will be handled in SBlendSpaceGridWidget so, whilst this
// gets called, it shouldn't really need to do anything.
void UBlendSpace::SnapSamplesToClosestGridPoint()
{
	if (BlendParameters[0].bSnapToGrid && BlendParameters[1].bSnapToGrid)
	{
		TArray<int32> ClosestSampleToGridPoint;

		const FVector GridMin(BlendParameters[0].Min, BlendParameters[1].Min, 0.0f);
		const FVector GridMax(BlendParameters[0].Max, BlendParameters[1].Max, 0.0f);
		const FVector GridRange(GridMax.X - GridMin.X, GridMax.Y - GridMin.Y, 0.0f);
		const FIntPoint NumGridDivisions(BlendParameters[0].GridNum, BlendParameters[1].GridNum);
		const FVector GridStep(GridRange.X / BlendParameters[0].GridNum, GridRange.Y / BlendParameters[1].GridNum, 0.0f);

		// Snap to nearest in normalized space - not depending on the units of the params (which may be completely different).
		const float GridRatio = GridStep.Y / GridStep.X;

		int32 NumGridPoints = (NumGridDivisions.X + 1) * (NumGridDivisions.Y + 1);
		ClosestSampleToGridPoint.Init(INDEX_NONE, NumGridPoints);

		// Find closest sample to grid point
		for (int32 GridIndex = 0; GridIndex < NumGridPoints; ++GridIndex)
		{
			const FVector& GridPoint = GetGridPosition(GridIndex);
			float SmallestDistanceSq = FLT_MAX;
			int32 Index = INDEX_NONE;

			for (int32 SampleIndex = 0; SampleIndex < SampleData.Num(); ++SampleIndex)
			{
				FBlendSample& BlendSample = SampleData[SampleIndex];
				FVector Delta = GridPoint - BlendSample.SampleValue;
				Delta.X *= GridRatio;
				const float DistanceSq = Delta.SizeSquared2D();
				if (DistanceSq < SmallestDistanceSq)
				{
					Index = SampleIndex;
					SmallestDistanceSq = DistanceSq;
				}
			}

			ClosestSampleToGridPoint[GridIndex] = Index;
		}

		// Find closest grid point to sample
		for (int32 SampleIndex = 0; SampleIndex < SampleData.Num(); ++SampleIndex)
		{
			FBlendSample& BlendSample = SampleData[SampleIndex];

			// Find closest grid point
			float SmallestDistanceSq = FLT_MAX;
			int32 Index = INDEX_NONE;
			for (int32 GridIndex = 0; GridIndex < NumGridPoints; ++GridIndex)
			{
				FVector Delta = GetGridPosition(GridIndex) - BlendSample.SampleValue;
				Delta.X *= GridRatio;
				const float DistanceSq = Delta.SizeSquared2D();
				if (DistanceSq < SmallestDistanceSq)
				{
					Index = GridIndex;
					SmallestDistanceSq = DistanceSq;
				}
			}

			// Only move the sample if it is also closest to the grid point
			if (Index != INDEX_NONE && ClosestSampleToGridPoint[Index] == SampleIndex)
			{
				for (int32 AxisIndex = 0; AxisIndex != 2; ++AxisIndex)
				{
					BlendSample.SampleValue[AxisIndex] = GetGridPosition(Index)[AxisIndex];
				}
			}
		}
	}
	else if (BlendParameters[0].bSnapToGrid || BlendParameters[1].bSnapToGrid)
	{
		// We only snap on one axis, but need to make sure we don't collapse samples on top of each
		// other. Just snap to the nearest grid value, unless it would result in a duplicate

		int32 AxisIndex = BlendParameters[0].bSnapToGrid ? 0 : 1;

		float GridMin = BlendParameters[AxisIndex].Min;
		float GridMax = BlendParameters[AxisIndex].Max;
		int32 GridNum = BlendParameters[AxisIndex].GridNum;
		float GridDelta = (GridMax - GridMin) / GridNum;

		for (int32 SampleIndex = 0; SampleIndex < SampleData.Num(); ++SampleIndex)
		{
			FBlendSample& BlendSample = SampleData[SampleIndex];
			FVector NewSampleValue = BlendSample.SampleValue;
			float FGridPosition = (NewSampleValue[AxisIndex] - GridMin) / GridDelta;
			int32 IGridPosition = FMath::Clamp((int)(FGridPosition + 0.5f), 0, GridNum + 1);
			NewSampleValue[AxisIndex] = GridMin + IGridPosition * GridDelta;

			bool bFoundOverlap = false;
			for (int32 SampleIndex1 = 0; SampleIndex1 != SampleData.Num(); ++SampleIndex1)
			{
				if (IsSameSamplePoint(NewSampleValue, SampleData[SampleIndex1].SampleValue))
				{
					bFoundOverlap = true;
					break;
				}
			}
			if (!bFoundOverlap)
			{
				BlendSample.SampleValue = NewSampleValue;
			}
		}
	}
}

void UBlendSpace::ClearBlendSpaceData()
{
	BlendSpaceData.Empty();
}

void UBlendSpace::SetBlendSpaceData(const TArray<FBlendSpaceTriangle>& Triangles)
{
	ClearBlendSpaceData();
	BlendSpaceData.Triangles = Triangles;
}

void UBlendSpace::SetBlendSpaceData(const TArray<FBlendSpaceSegment>& Segments)
{
	ClearBlendSpaceData();
	BlendSpaceData.Segments = Segments;
}

void UBlendSpace::ResampleData()
{
	ClearBlendSpaceData();

	ValidateSampleData();

	FBox AABB(ForceInit);
	for (const FBlendSample& Sample : SampleData)
	{
		// Add X value from sample (this is the only valid value to be set for 1D blend spaces / aim offsets
		if (Sample.bIsValid)
		{
			AABB += Sample.SampleValue;
		}
	}

	DimensionIndices.Reset(3);
	if (AABB.GetExtent().X > 0.0f)
	{
		DimensionIndices.Push(0);
	}
	if (AABB.GetExtent().Y > 0.0f)
	{
		DimensionIndices.Push(1);
	}
	if (AABB.GetExtent().Z > 0.0f)
	{
		DimensionIndices.Push(2);
	}

	// Handle the situation where there is just one point
	if (DimensionIndices.Num() == 0)
	{
		DimensionIndices.Push(0);
	}

	if (DimensionIndices.Num() == 1)
	{
		ResampleData1D();
	}
	else if (DimensionIndices.Num() == 2)
	{
		ResampleData2D();
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "Found %d dimensions from the samples", DimensionIndices.Num());
	}
}


void UBlendSpace::ResampleData1D()
{
	check(DimensionIndices.Num() == 1);
	FLineElementGenerator LineElementGenerator;

	int32 Index0 = DimensionIndices[0];

	const FBlendParameter& BlendParameter = GetBlendParameter(Index0);
	LineElementGenerator.Init(BlendParameter);

	UE_LOGF(LogAnimation, Log, "Resampling data in 1D - %d samples", SampleData.Num());

	if (SampleData.Num())
	{
		for (int32 SampleIndex = 0; SampleIndex < SampleData.Num(); ++SampleIndex)
		{
			const FBlendSample& Sample = SampleData[SampleIndex];
			// Add X value from sample (this is the only valid value to be set for 1D blend spaces / aim offsets
			if (Sample.bIsValid)
			{
				LineElementGenerator.SamplePointList.Add(FLineVertex(Sample.SampleValue[Index0], SampleIndex));
			}
		}

		LineElementGenerator.Process();

		if (bInterpolateUsingGrid)
		{
			LineElementGenerator.CalculateEditorElements();
			FillupGridElements(LineElementGenerator.EditorElements, DimensionIndices);
			ClearBlendSpaceData();
		}
		else
		{
			SetBlendSpaceData(LineElementGenerator.CalculateSegments());
			EmptyGridElements();
		}
	}
}


void UBlendSpace::ResampleData2D()
{
	check(DimensionIndices.Num() == 2);

	// TODO for now, Index0 will always be 0 and Index1 will always be 1, since we don't support 3D.
	// However, if/when we support authoring using a third dimension, we could get here with any 2D
	// combination - e.g. XY, XZ or YZ. Then we can either make the triangulation code handle the
	// indexing, or tweak the triangles going into and out of it.
	int32 Index0 = DimensionIndices[0];
	int32 Index1 = DimensionIndices[1];

	// clear first
	FDelaunayTriangleGenerator DelaunayTriangleGenerator;

	// you don't like to overwrite the link here (between visible points vs sample points, 
	// so allow this if no triangle is generated
	const FBlendParameter& BlendParamX = GetBlendParameter(Index0);
	const FBlendParameter& BlendParamY = GetBlendParameter(Index1);
	FBlendSpaceGrid	BlendSpaceGrid;
	BlendSpaceGrid.SetGridInfo(BlendParamX, BlendParamY);
	DelaunayTriangleGenerator.SetGridBox(BlendParamX, BlendParamY);

	EmptyGridElements();

	UE_LOGF(LogAnimation, Log, "Resampling data in 2D - %d samples", GetNumberOfBlendSamples());
	if (GetNumberOfBlendSamples())
	{
		bool bAllSamplesValid = true;
		for (int32 SampleIndex = 0; SampleIndex < GetNumberOfBlendSamples(); ++SampleIndex)
		{
			const FBlendSample& Sample = GetBlendSample(SampleIndex);

			// Do not add invalid sample points (user will need to correct them to be incorporated into the blendspace)
			if (Sample.bIsValid)
			{
				DelaunayTriangleGenerator.AddSamplePoint(FVector2D(Sample.SampleValue[Index0], Sample.SampleValue[Index1]), SampleIndex);
			}
		}

		// triangulate
		DelaunayTriangleGenerator.Triangulate(PreferredTriangulationDirection);

		if (bInterpolateUsingGrid)
		{
			// once triangulated, generate grid
			const TArray<FVertex>& Points = DelaunayTriangleGenerator.GetSamplePointList();
			const TArray<FTriangle*>& Triangles = DelaunayTriangleGenerator.GetTriangleList();
			BlendSpaceGrid.GenerateGridElements(Points, Triangles);

			// now fill up grid elements in BlendSpace using this Element information
			if (Triangles.Num() > 0)
			{
				const TArray<FEditorElement>& GridElements = BlendSpaceGrid.GetElements();
				FillupGridElements(GridElements, DimensionIndices);
			}

			ClearBlendSpaceData();
		}
		else
		{
			SetBlendSpaceData(DelaunayTriangleGenerator.CalculateTriangles());
			EmptyGridElements();
		}

	}
}

#endif // WITH_EDITOR

void FBlendSpaceData::GetSamples1D(
	TArray<FWeightedBlendSample>& OutWeightedSamples,
	const TArray<int32>&          InDimensionIndices,
	const FVector&                InNormalizedSamplePosition,
	int32&                        InOutSegmentIndex) const
{
	check(InDimensionIndices.Num() == 1);
	OutWeightedSamples.Reset(2);

	if (!Segments.Num())
	{
		return;
	}
	else if (Segments.Num() == 1 && Segments[0].SampleIndices[0] == Segments[0].SampleIndices[1])
	{
		OutWeightedSamples.Push(FWeightedBlendSample(Segments[0].SampleIndices[0], 1.0f));
		return;
	}

	int32 Index0 = InDimensionIndices[0];
	float P = InNormalizedSamplePosition[Index0];

#ifdef DEBUG_LOG_BLENDSPACE_TRIANGULATION
	static int32 Count = 0;
	UE_LOGF(LogAnimation, Warning, "%d Starting segment search with cached index %d", Count++, InOutSegmentIndex);
#endif

	if (InOutSegmentIndex < 0 || InOutSegmentIndex >= Segments.Num())
	{
		InOutSegmentIndex = Segments.Num() / 2;
	}

	for (int32 Attempt = 0; Attempt != Segments.Num(); ++Attempt)
	{
#ifdef DEBUG_LOG_BLENDSPACE_TRIANGULATION
		UE_LOGF(LogAnimation, Warning, "Segment index: %d", InOutSegmentIndex);
#endif
		const FBlendSpaceSegment& Segment = Segments[InOutSegmentIndex];
		if (P < Segment.Vertices[0])
		{
			// Need to go left
			if (InOutSegmentIndex > 0)
			{
				--InOutSegmentIndex;
			}
			else
			{
				OutWeightedSamples.Push(FWeightedBlendSample(Segment.SampleIndices[0], 1.0f));
				return;
			}
		}
		else if (P > Segment.Vertices[1])
		{
			// Need to go right
			if (InOutSegmentIndex < Segments.Num() - 1)
			{
				++InOutSegmentIndex;
			}
			else
			{
				OutWeightedSamples.Push(FWeightedBlendSample(Segment.SampleIndices[1], 1.0f));
				return;
			}
		}
		else
		{
			// We're in the segment
			float P0 = Segment.Vertices[0];
			float P1 = Segment.Vertices[1];
			float Frac = FMath::Clamp((P - P0) / (P1 - P0), 0.0f, 1.0f); // Robust to dividing by zero
			OutWeightedSamples.Push(FWeightedBlendSample(Segment.SampleIndices[0], 1.0f - Frac));
			OutWeightedSamples.Push(FWeightedBlendSample(Segment.SampleIndices[1], Frac));
			return;
		}
	}
	UE_LOGF(LogAnimation, Warning, "Unable to find BlendSpace segment");
}

void FBlendSpaceData::GetSamples2D(
	TArray<FWeightedBlendSample>& OutWeightedSamples,
	const TArray<int32>&          InDimensionIndices,
	const FVector&                InNormalizedSamplePosition,
	int32&                        InOutTriangleIndex) const
{
	if (Triangles.Num() == 0)
	{
		return;
	}

	check(InDimensionIndices.Num() == 2);
	OutWeightedSamples.Reset(3);

	// Do an incremental search by tracking through the triangle edges (with the normals). This will
	// be guaranteed to work since the overall triangulation region is convex. Note that the
	// triangulation may not include the query point, so if we're not in any triangle we need to
	// return the closest point in the triangulated region.
	int32 Index0 = InDimensionIndices[0];
	int32 Index1 = InDimensionIndices[1];
	FVector2D P(InNormalizedSamplePosition[Index0], InNormalizedSamplePosition[Index1]);

	// Special case for when there's a single triangle and it is degenerate
	if (Triangles.Num() == 1)
	{
		const FBlendSpaceTriangle& Triangle = Triangles[0];
		if (Triangle.SampleIndices[0] == Triangle.SampleIndices[1])
		{
			// Single point
			OutWeightedSamples.Push(FWeightedBlendSample(Triangle.SampleIndices[0], 1.0f));
			return;
		}
		else if (Triangle.SampleIndices[1] == Triangle.SampleIndices[2])
		{
			// Two points - blend linearly
			FVector2D Delta = Triangle.Vertices[1] - Triangle.Vertices[0];
			float T = ((P - Triangle.Vertices[0]) | Delta) / Delta.SizeSquared();
			if (T < 0)
			{
				OutWeightedSamples.Push(FWeightedBlendSample(Triangle.SampleIndices[0], 1.0f));
			}
			else if (T > 1.0f)
			{
				OutWeightedSamples.Push(FWeightedBlendSample(Triangle.SampleIndices[1], 1.0f));
			}
			else
			{
				OutWeightedSamples.Push(FWeightedBlendSample(Triangle.SampleIndices[0], 1.0f - T));
				OutWeightedSamples.Push(FWeightedBlendSample(Triangle.SampleIndices[1], T));
			}
			return;
		}
	}

#ifdef DEBUG_LOG_BLENDSPACE_TRIANGULATION
	static int32 Count = 0;
	UE_LOGF(LogAnimation, Warning, "%d Starting triangulation search with cached index %d", Count++, InOutTriangleIndex);
#endif

	// Where available, start from the the previous/cached result. Also see
	// https://www.cl.cam.ac.uk/techreports/UCAM-CL-TR-728.pdf for a method which also works for
	// non-Delaunay triangulations (might be useful if there are precision problems).
	if (InOutTriangleIndex < 0 || InOutTriangleIndex >= Triangles.Num())
	{
		InOutTriangleIndex = Triangles.Num() / 2;
	}
	// We should never need more than the number of triangles - if we do then we got caught in a
	// loop somehow
	for (int32 Attempt = 0 ; Attempt != Triangles.Num() ; ++Attempt)
	{
#ifdef DEBUG_LOG_BLENDSPACE_TRIANGULATION
		UE_LOGF(LogAnimation, Warning, "Triangle index: %d", InOutTriangleIndex);
#endif
		const FBlendSpaceTriangle* Triangle = &Triangles[InOutTriangleIndex];
		// Look for the edge which has the target point most outside it
		float LargestDistance = UE_KINDA_SMALL_NUMBER;
		int32 LargestEdgeIndex = INDEX_NONE;
		for (int32 VertexIndex = 0; VertexIndex != FBlendSpaceTriangle::NUM_VERTICES; ++VertexIndex)
		{
			FVector2D Corner = Triangle->Vertices[VertexIndex];
			FVector2D EdgeNormal = Triangle->EdgeInfo[VertexIndex].Normal;
			float Distance = (P - Corner) | EdgeNormal;
			if (Distance > LargestDistance)
			{
				LargestDistance = Distance;
				LargestEdgeIndex = VertexIndex;
			}
		}
		if (LargestEdgeIndex < 0)
		{
			// Point is inside this triangle
			FVector Weights = FMath::GetBaryCentric2D(P, Triangle->Vertices[0], Triangle->Vertices[1], Triangle->Vertices[2]);
			OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[0], Weights[0]));
			OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[1], Weights[1]));
			OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[2], Weights[2]));
			return;
		}
		if (Triangle->EdgeInfo[LargestEdgeIndex].NeighbourTriangleIndex < 0)
		{
			// We're being directed to go outside the perimeter (which is convex). We will "walk"
			// around the perimeter until either:
			// 1. The point projected onto the edge is not at an extreme, or
			// 2. The direction changes
			// Note that the Triangle pointer will be updated during this walk
			int32 OrigDir = -1; // will be 0 or 1 when we get going
			int32 StartIndex = LargestEdgeIndex;
			int32 EndIndex = (StartIndex + 1) % FBlendSpaceTriangle::NUM_VERTICES;

			// Should only need a few steps
			int PrevTriangleIndex = InOutTriangleIndex;
			for (int32 PerimeterAttempt = 0; PerimeterAttempt != Triangles.Num(); ++PerimeterAttempt)
			{
#ifdef DEBUG_LOG_BLENDSPACE_TRIANGULATION
				UE_LOGF(LogAnimation, Warning, "Perimeter Triangle index: %d", InOutTriangleIndex);
#endif
				const FBlendSpaceTriangleEdgeInfo& EdgeInfo = Triangle->EdgeInfo[StartIndex];
				FVector2D Start = Triangle->Vertices[StartIndex];
				FVector2D End = Triangle->Vertices[EndIndex];
				FVector2D StartToEnd = End - Start;
				FVector2D StartToPoint = P - Start;
				// Parametric distance along the StartToEnd line
				float T = (StartToEnd | StartToPoint) / (StartToEnd | StartToEnd);

				// Check to see if we're (projected) on a perimeter segment
				if (T >= 0.0f && T <= 1.0f)
				{
					OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[StartIndex], 1.0f - T));
					OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[EndIndex], T));
					return;
				}

				// Check for a change in direction
				if (OrigDir == -1)
				{
					OrigDir = T > 1.0f ? 1 : 0;
				}
				int32 Dir = T > 1.0f ? 1 : 0;
				if (OrigDir != Dir)
				{
					if (Dir == 0)
						OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[StartIndex], 1.0f - T));
					else
						OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[EndIndex], T));
					// If we flipped direction then we only found out by moving to a new triangle.
					// If we cache the final triangle then next time we'll flip to the other
					// triangle. That's fine, but confusing when debugging! Instead, use the
					// previous triangle as the cache.
					InOutTriangleIndex = PrevTriangleIndex;
					return;
				}

				// Walk to the next triangle around the perimeter, but keep track of where we came from
				PrevTriangleIndex = InOutTriangleIndex;
				InOutTriangleIndex = EdgeInfo.AdjacentPerimeterTriangleIndices[Dir];
				if (InOutTriangleIndex < 0)
				{
					// This can happen if there weren't many triangles, in which case there won't be
					// another triangle in this direction.
					if (Dir == 0)
						OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[StartIndex], 1.0f - T));
					else
						OutWeightedSamples.Push(FWeightedBlendSample(Triangle->SampleIndices[EndIndex], T));
					InOutTriangleIndex = PrevTriangleIndex;
					return;
				}
				Triangle = &Triangles[InOutTriangleIndex];
				StartIndex = EdgeInfo.AdjacentPerimeterVertexIndices[Dir];
				EndIndex = (StartIndex + 1) % FBlendSpaceTriangle::NUM_VERTICES;
			}
		}
		else
		{
			// Investigate the new triangle
			const FBlendSpaceTriangleEdgeInfo& EdgeInfo = Triangle->EdgeInfo[LargestEdgeIndex];
			InOutTriangleIndex = EdgeInfo.NeighbourTriangleIndex;
		}
	}
	UE_LOGF(LogAnimation, Warning, "Unable to find BlendSpace triangle");
}

float FBlendSample::GetSamplePlayLength() const
{
	if (Animation)
	{
		if (bUseSingleFrameForBlending)
		{
			return Animation->GetSamplingFrameRate().AsInterval();
		}

		return Animation->GetPlayLength();
	}

	return 0.f;
}

void FBlendSpaceData::GetSamples(
	TArray<FWeightedBlendSample>& OutWeightedSamples,
	const TArray<int32>&          InDimensionIndices,
	const FVector&                InNormalizedSamplePosition,
	int32&                        InOutTriangulationIndex) const
{
	switch (InDimensionIndices.Num())
	{
	case 1:
		return GetSamples1D(OutWeightedSamples, InDimensionIndices, InNormalizedSamplePosition, InOutTriangulationIndex);
	case 2:
		return GetSamples2D(OutWeightedSamples, InDimensionIndices, InNormalizedSamplePosition, InOutTriangulationIndex);
	default:
		UE_LOGF(LogAnimation, Warning, "Unhandled number of dimensions in BlendSpace: %d", InDimensionIndices.Num());
		OutWeightedSamples.Reset();
		break;
	}
}


#undef LOCTEXT_NAMESPACE 

