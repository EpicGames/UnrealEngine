// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BlendSpace1D.cpp: 1D BlendSpace functionality
=============================================================================*/ 

#include "Animation/BlendSpace1D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSpace1D)

UBlendSpace1D::UBlendSpace1D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DimensionIndices = { 0 };
}

EBlendSpaceAxis UBlendSpace1D::GetAxisToScale() const
{
	return bScaleAnimation ? BSA_X : BSA_None;
}

#if WITH_EDITOR

void UBlendSpace1D::SnapSamplesToClosestGridPoint()
{
	if (!BlendParameters[0].bSnapToGrid)
		return;

	TArray<int32> ClosestSampleToGridPoint;

	const float GridMin = BlendParameters[0].Min;
	const float GridMax = BlendParameters[0].Max;
	const float GridRange = GridMax - GridMin;
	const int32 NumGridPoints = BlendParameters[0].GridNum + 1;
	const float GridStep = GridRange / BlendParameters[0].GridNum;

	ClosestSampleToGridPoint.Init(INDEX_NONE, NumGridPoints);

	// Find closest sample to grid point
	for (int32 PointIndex = 0; PointIndex < NumGridPoints; ++PointIndex)
	{
		const float GridPoint = GetGridPosition(PointIndex)[0];
		float SmallestDistance = FLT_MAX;
		int32 Index = INDEX_NONE;

		for (int32 SampleIndex = 0; SampleIndex < SampleData.Num(); ++SampleIndex)
		{
			FBlendSample& BlendSample = SampleData[SampleIndex];
			const float Distance = FMath::Abs(GridPoint - BlendSample.SampleValue[0]);
			if (Distance < SmallestDistance)
			{
				Index = SampleIndex;
				SmallestDistance = Distance;
			}
		}

		ClosestSampleToGridPoint[PointIndex] = Index;
	}

	// Find closest grid point to sample
	for (int32 SampleIndex = 0; SampleIndex < SampleData.Num(); ++SampleIndex)
	{
		FBlendSample& BlendSample = SampleData[SampleIndex];

		// Find closest grid point
		float SmallestDistance = FLT_MAX;
		int32 Index = INDEX_NONE;
		for (int32 PointIndex = 0; PointIndex < NumGridPoints; ++PointIndex)
		{
			const float Distance = FMath::Abs(GetGridPosition(PointIndex)[0] - BlendSample.SampleValue[0]);
			if (Distance < SmallestDistance)
			{
				Index = PointIndex;
				SmallestDistance = Distance;
			}
		}

		// Only move the sample if it is also closest to the grid point
		if (Index != INDEX_NONE && ClosestSampleToGridPoint[Index] == SampleIndex)
		{
			BlendSample.SampleValue[0] = GetGridPosition(Index)[0];
		}
	}
}

#endif // WITH_EDITOR



