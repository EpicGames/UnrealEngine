// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/BlendSpace1DHelpers.h"
#include "Animation/BlendSpace.h"

#define LOCTEXT_NAMESPACE "AnimationBlendSpace1DHelpers"

bool FLineElement::PopulateElement(const float ElementPosition, FEditorElement& InOutElement) const
{	
	// If the element is left of the line element
	if (ElementPosition < Start.Position)
	{
		// Element can only be left of a point if it is the first one (otherwise line is incorrect)
		if (!bIsFirst)
		{
			return false;
		}

		InOutElement.Indices[0] = Start.Index;
		InOutElement.Weights[0] = 1.0f;
		return true;
	}
	// If the element is right of the line element
	else if (ElementPosition > End.Position)
	{
		// Element can only be right of a point if it is the last one (otherwise line is incorrect)
		if (!bIsLast)
		{
			return false;
		}
		InOutElement.Indices[0] = End.Index;
		InOutElement.Weights[0] = 1.0f;
		return true;
	}
	else
	{
		// If the element is between the start/end point of the line weight according to where it's closest to
		InOutElement.Indices[0] = End.Index;
		InOutElement.Weights[0] = (ElementPosition - Start.Position) / Range;

		InOutElement.Indices[1] = Start.Index;
		InOutElement.Weights[1] = (1.0f - InOutElement.Weights[0]);		
		return true;
	}
}

void FLineElementGenerator::Init(const FBlendParameter& BlendParameter)
{
	SamplePointList.Reset();
	MinGridValue = BlendParameter.Min;
	MaxGridValue = BlendParameter.Max;
	NumGridPoints = BlendParameter.GridNum + 1;
	NumGridDivisions = BlendParameter.GridNum;
}

void FLineElementGenerator::Process()
{
	if (SamplePointList.Num() > 1)
	{
		// Sort points according to position value
		SamplePointList.Sort([](const FLineVertex& A, const FLineVertex& B) { return A.Position < B.Position; });
	}
}

TArray<FBlendSpaceSegment> FLineElementGenerator::CalculateSegments() const
{
	TArray<FBlendSpaceSegment> Segments;
	Segments.Reserve(SamplePointList.Num() > 1 ? SamplePointList.Num() - 1 : 0);

	// Only create lines if we have more than one point to draw between
	if (SamplePointList.Num() == 0)
	{
		return Segments;
	}
	else if (SamplePointList.Num() == 1)
	{
		FBlendSpaceSegment NewSegment;
		NewSegment.Vertices[0] = 0.0f;
		NewSegment.Vertices[1] = 1.0f;
		NewSegment.SampleIndices[0] = NewSegment.SampleIndices[1] = SamplePointList[0].Index;
		Segments.Add(NewSegment);
	}
	else
	{
		// Generate lines between sampling points from start to end (valid since they were sorted)	
		for (int32 PointIndex = 0; PointIndex < SamplePointList.Num() - 1; ++PointIndex)
		{
			const int32 EndPointIndex = PointIndex + 1;
			FBlendSpaceSegment NewSegment;
			NewSegment.Vertices[0] = (SamplePointList[PointIndex].Position - MinGridValue) / (MaxGridValue - MinGridValue);
			NewSegment.Vertices[1] = (SamplePointList[EndPointIndex].Position - MinGridValue) / (MaxGridValue - MinGridValue);
			NewSegment.SampleIndices[0] = SamplePointList[PointIndex].Index;
			NewSegment.SampleIndices[1] = SamplePointList[EndPointIndex].Index;
			Segments.Add(NewSegment);
		}
	}
	return Segments;
}


void FLineElementGenerator::CalculateEditorElements()
{	
	// Always clear line elements
	LineElements.Empty(SamplePointList.Num() > 1 ? SamplePointList.Num() - 1 : 0);
	
	// Only create lines if we have more than one point to draw between
	if (SamplePointList.Num() > 1)
	{
		// Generate lines between sampling points from start to end (valid since they were sorted)	
		for (int32 PointIndex = 0; PointIndex < SamplePointList.Num() - 1; ++PointIndex)
		{
			const int32 EndPointIndex = PointIndex + 1;
			LineElements.Add(FLineElement(SamplePointList[PointIndex], SamplePointList[EndPointIndex]));
		}

		// Set first and last sample flags (safe because at this point there always at least one sample)
		LineElements[0].bIsFirst = true;
		LineElements.Last().bIsLast = true;
	}	
	
	// Number of division between grid edges
	const float GridRange = MaxGridValue - MinGridValue;
	const float GridStep = GridRange / NumGridDivisions;

	// Initialize editor elements to required number of points
	EditorElements.Empty(NumGridPoints);
	EditorElements.AddDefaulted(NumGridPoints);

	if (LineElements.Num() == 0)
	{
		// Since we did not generate any lines all the samples should correspond to the first sample and fully weighted to 1
		for (FEditorElement& Element : EditorElements)
		{
			Element.Indices[0] = 0;
			Element.Weights[0] = 1.0f;
		}
	}
	else
	{
		for (int32 ElementIndex = 0; ElementIndex < NumGridPoints; ++ElementIndex)
		{
			FEditorElement& Element = EditorElements[ElementIndex];
			const float ElementGridPosition = (GridStep * ElementIndex) + MinGridValue;

			// Try and populate the editor element
			bool bPopulatedElement = false;
			for (const FLineElement& LineElement : LineElements)
			{
				bPopulatedElement |= LineElement.PopulateElement(ElementGridPosition, Element);
				if (bPopulatedElement)
				{
					break;
				}
			}
			
			// Ensure that the editor element is populated using the available sample data
			check(bPopulatedElement);
		}
	}
}

#undef LOCTEXT_NAMESPACE
