// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FLineVertex
{
	float Position;
	int32 Index;
	FLineVertex(const float InPosition, const int32 InIndex) : Position(InPosition), Index(InIndex) {}
};

struct FLineElement
{
	// Explicit constructor since we need to populate the Range value
	explicit FLineElement(const FLineVertex& InStart, const FLineVertex& InEnd) : Start(InStart), End(InEnd), bIsFirst(false), bIsLast(false)
	{
		Range = End.Position - Start.Position;
	}

	bool PopulateElement(const float ElementPosition, struct FEditorElement& InOutElement) const;
	bool IsBlendInputOnLine(const FVector& BlendInput) const
	{
		return (BlendInput.X >= Start.Position) && (BlendInput.X <= End.Position);
	}

	const FLineVertex Start;
	const FLineVertex End;
	bool bIsFirst;
	bool bIsLast;
	float Range;
};

/** Generates a line list between the supplied sample points to
aid blend space sample generation */
class FLineElementGenerator
{
public:
	void Init(const struct FBlendParameter& BlendParameter);

	void Process();

	/** Populates EditorElements based on the Sample points previously supplied to AddSamplePoint */
	void CalculateEditorElements();
	
	/**
	 * Converts from our internal representation to the runtime segments
	 */
	TArray<struct FBlendSpaceSegment> CalculateSegments() const;

	/**
	* Data Structure for line generation
	* SamplePointList is the input data
	*/
	TArray<FLineVertex> SamplePointList;

	/** Editor elements generated by CalculateEditorElements */
	TArray<struct FEditorElement> EditorElements;

private:

	/** Defines the range of the editor */
	float MinGridValue;
	float MaxGridValue;

	/** Number of points that we have to generate FEditorElements for */
	int32 NumGridPoints;
	int32 NumGridDivisions;

	/** Line elements generated from the given samples */
	TArray<FLineElement> LineElements;
};
