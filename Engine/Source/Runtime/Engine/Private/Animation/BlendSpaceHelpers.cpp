// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/BlendSpaceHelpers.h"

#define BLENDSPACE_MINSAMPLE	3

#define LOCTEXT_NAMESPACE "AnimationBlendSpaceHelpers"

// Stores an edge or triangle index, and distance to a point, in anticipation of sorting.
struct FIndexAndDistance
{
	int32 Index;
	double Distance;
	FIndexAndDistance(int32 InIndex, double InDistance)
		: Index(InIndex), Distance(InDistance) {}
};

void FDelaunayTriangleGenerator::SetGridBox(const FBlendParameter& BlendParamX, const FBlendParameter& BlendParamY)
{
	FVector2D Min(BlendParamX.Min, BlendParamY.Min);
	FVector2D Max(BlendParamX.Max, BlendParamY.Max);
	FVector2D Mid = (Min + Max) * 0.5f;
	FVector2D Range = Max - Min;
	Range.X = FMath::Max(Range.X, UE_DELTA);
	Range.Y = FMath::Max(Range.Y, UE_DELTA);

	GridMin = Mid - Range * 0.5f;
	RecipGridSize = FVector2D(1.0, 1.0) / Range;
}

void FDelaunayTriangleGenerator::EmptyTriangles()
{
	for (int32 TriangleIndex = 0; TriangleIndex < TriangleList.Num(); ++TriangleIndex)
	{
		delete TriangleList[TriangleIndex];
	}

	TriangleList.Empty();
}
void FDelaunayTriangleGenerator::EmptySamplePoints()
{
	SamplePointList.Empty();
}

void FDelaunayTriangleGenerator::Reset()
{
	EmptyTriangles();
	EmptySamplePoints();
}

void FDelaunayTriangleGenerator::AddSamplePoint(const FVector2D& NewPoint, const int32 SampleIndex)
{
	checkf(!SamplePointList.Contains(NewPoint), TEXT("Found duplicate points in blendspace"));
	SamplePointList.Add(FVertex(NewPoint, SampleIndex));
}

void FDelaunayTriangleGenerator::Triangulate(EPreferredTriangulationDirection PreferredTriangulationDirection)
{
	if (SamplePointList.Num() == 0)
	{
		return;
	}
	else if (SamplePointList.Num() == 1)
	{
		// degenerate case 1
		FTriangle Triangle(&SamplePointList[0]);
		AddTriangle(Triangle);
	}
	else if (SamplePointList.Num() == 2)
	{
		// degenerate case 2
		FTriangle Triangle(&SamplePointList[0], &SamplePointList[1]);
		AddTriangle(Triangle);
	}
	else
	{
		SortSamples();

		// first choose first 3 points
		for (int32 I = 2; I<SamplePointList.Num(); ++I)
		{
			GenerateTriangles(SamplePointList, I + 1);
		}

		// degenerate case 3: many points all collinear or coincident
		if (TriangleList.Num() == 0)
		{
			if (AllCoincident(SamplePointList))
			{
				// coincident case - just create one triangle
				FTriangle Triangle(&SamplePointList[0]);
				AddTriangle(Triangle);
			}
			else
			{
				// collinear case: create degenerate triangles between pairs of points
				for (int32 PointIndex = 0; PointIndex < SamplePointList.Num() - 1; ++PointIndex)
				{
					FTriangle Triangle(&SamplePointList[PointIndex], &SamplePointList[PointIndex + 1]);
					AddTriangle(Triangle);
				}
			}
		}
	}
	AdjustEdgeDirections(PreferredTriangulationDirection);
}

static int32 GetCandidateEdgeIndex(const FTriangle* Triangle)
{
	bool bFoundVert = false;
	bool bFoundHor = false;
	bool bFoundSlope = false;
	int32 SlopeIndex = 0;
	for (int32 VertexIndex = 0; VertexIndex != 3; ++VertexIndex)
	{
		int32 VertexIndexNext = (VertexIndex + 1) % 3;
		if (Triangle->Vertices[VertexIndex]->Position.X == Triangle->Vertices[VertexIndexNext]->Position.X)
		{
			bFoundVert = true;
		}
		else if (Triangle->Vertices[VertexIndex]->Position.Y == Triangle->Vertices[VertexIndexNext]->Position.Y)
		{
			bFoundHor = true;
		}
		else
		{
			bFoundSlope = true;
			SlopeIndex = VertexIndex;
		}
	}
	if (bFoundVert && bFoundHor && bFoundSlope)
	{
		return SlopeIndex;
	}
	else
	{
		return -1;
	}
}

void FDelaunayTriangleGenerator::AdjustEdgeDirections(EPreferredTriangulationDirection PreferredTriangulationDirection)
{
	if (PreferredTriangulationDirection == EPreferredTriangulationDirection::None)
	{
		return;
	}

	for (int32 TriangleIndex0 = 0; TriangleIndex0 != TriangleList.Num(); ++TriangleIndex0)
	{
		FTriangle* Triangle0 = TriangleList[TriangleIndex0];

		// Check if it is axis aligned and if so which is the hypotenuse edge
		int32 EdgeIndex0 = GetCandidateEdgeIndex(Triangle0);
		if (EdgeIndex0 >= 0)
		{
			int32 EdgeIndexNext0 = (EdgeIndex0 + 1) % 3;
			// Only continue if we would want to flip this edge
			FVector2D EdgeDir = Triangle0->Vertices[EdgeIndexNext0]->Position - Triangle0->Vertices[EdgeIndex0]->Position;
			FVector2D EdgePos = (Triangle0->Vertices[EdgeIndexNext0]->Position + Triangle0->Vertices[EdgeIndex0]->Position) * 0.5f;

			// Slope is true if we go up from left to right
			bool bUpSlope = EdgeDir.X * EdgeDir.Y > 0.0f;
			// Desired slope depends on the quadrant
			bool bDesiredUpSlope = PreferredTriangulationDirection == EPreferredTriangulationDirection::Tangential ? EdgePos.X * EdgePos.Y < 0.0f : EdgePos.X * EdgePos.Y >= 0.0f;

			if (bUpSlope == bDesiredUpSlope)
			{
				continue;
			}

			// Look for a triangle that is also axis aligned and facing this sloping edge
			int32 SampleIndexStart = Triangle0->Vertices[EdgeIndex0]->SampleIndex;
			int32 SampleIndexEnd = Triangle0->Vertices[(EdgeIndex0 + 1) % 3]->SampleIndex;
			int32 EdgeIndex1;
			int32 TriangleIndex1 = FindTriangleIndexWithEdge(SampleIndexEnd, SampleIndexStart, &EdgeIndex1);
			if (TriangleIndex1 < 0)
			{
				continue;
			}
			check(EdgeIndex1 >= 0);
			FTriangle* Triangle1 = TriangleList[TriangleIndex1];
			if (GetCandidateEdgeIndex(Triangle1) == EdgeIndex1)
			{
				// Flip the edge across the pair of triangles. Note that after this the triangles
				// will contain incorrect edge info, but we don't need that any more.
				int32 EdgeIndexPrev1 = (EdgeIndex1 + 2) % 3;
				int32 EdgeIndexNext1 = (EdgeIndex1 + 1) % 3;
				int32 EdgeIndexPrev0 = (EdgeIndex0 + 2) % 3;

				Triangle0->Vertices[EdgeIndexNext0] = Triangle1->Vertices[EdgeIndexPrev1];
				Triangle1->Vertices[EdgeIndexNext1] = Triangle0->Vertices[EdgeIndexPrev0];
				Triangle0->UpdateCenter();
				Triangle1->UpdateCenter();
			}
		}
	}
}

void FDelaunayTriangleGenerator::SortSamples()
{
	// Populate sorting array with sample points and their original (blend space -> sample data) indices

	struct FComparePoints
	{
		FORCEINLINE bool operator()( const FVertex& A, const FVertex& B ) const
		{
			// the sorting happens from -> +X, -> +Y,  -> for now ignore Z ->+Z
			if( A.Position.Y == B.Position.Y ) // same, then compare Y
			{
				return A.Position.X < B.Position.X;
			}
			return A.Position.Y < B.Position.Y;			
		}
	};
	// sort all points
	SamplePointList.Sort( FComparePoints() );
}

/** 
* The key function in Delaunay Triangulation
* return true if the TestPoint is WITHIN the triangle circumcircle
*	http://en.wikipedia.org/wiki/Delaunay_triangulation 
*/
FDelaunayTriangleGenerator::ECircumCircleState FDelaunayTriangleGenerator::GetCircumcircleState(const FTriangle* T, const FVertex& TestPoint)
{
	const int32 NumPointsPerTriangle = 3;

	// First off, normalize all the points
	FVector2D NormalizedPositions[NumPointsPerTriangle];
	
	// Unrolled loop
	NormalizedPositions[0] = (T->Vertices[0]->Position - GridMin) * RecipGridSize;
	NormalizedPositions[1] = (T->Vertices[1]->Position - GridMin) * RecipGridSize;
	NormalizedPositions[2] = (T->Vertices[2]->Position - GridMin) * RecipGridSize;

	const FVector2D NormalizedTestPoint = ( TestPoint.Position - GridMin ) * RecipGridSize;

	// ignore Z, eventually this has to be on plane
	// http://en.wikipedia.org/wiki/Delaunay_triangulation - determinant
	const double M00 = NormalizedPositions[0].X - NormalizedTestPoint.X;
	const double M01 = NormalizedPositions[0].Y - NormalizedTestPoint.Y;
	const double M02 = NormalizedPositions[0].X * NormalizedPositions[0].X - NormalizedTestPoint.X * NormalizedTestPoint.X
		+ NormalizedPositions[0].Y*NormalizedPositions[0].Y - NormalizedTestPoint.Y * NormalizedTestPoint.Y;

	const double M10 = NormalizedPositions[1].X - NormalizedTestPoint.X;
	const double M11 = NormalizedPositions[1].Y - NormalizedTestPoint.Y;
	const double M12 = NormalizedPositions[1].X * NormalizedPositions[1].X - NormalizedTestPoint.X * NormalizedTestPoint.X
		+ NormalizedPositions[1].Y * NormalizedPositions[1].Y - NormalizedTestPoint.Y * NormalizedTestPoint.Y;

	const double M20 = NormalizedPositions[2].X - NormalizedTestPoint.X;
	const double M21 = NormalizedPositions[2].Y - NormalizedTestPoint.Y;
	const double M22 = NormalizedPositions[2].X * NormalizedPositions[2].X - NormalizedTestPoint.X * NormalizedTestPoint.X
		+ NormalizedPositions[2].Y * NormalizedPositions[2].Y - NormalizedTestPoint.Y * NormalizedTestPoint.Y;

	const double Det = M00*M11*M22+M01*M12*M20+M02*M10*M21 - (M02*M11*M20+M01*M10*M22+M00*M12*M21);
	
	// When the vertices are sorted in a counterclockwise order, the determinant is positive if and only if Testpoint lies inside the circumcircle of T.
	if (Det < 0.0)
	{
		return ECCS_Outside;
	}
	else
	{
		// On top of the triangle edge
		if (FMath::IsNearlyZero(Det, UE_DOUBLE_SMALL_NUMBER))
		{
			return ECCS_On;
		}
		else
		{
			return ECCS_Inside;
		}
	}
}

int32 FDelaunayTriangleGenerator::FindTriangleIndexWithEdge(int32 SampleIndex0, int32 SampleIndex1, int32* VertexIndex) const
{
	for (int32 TriangleIndex = 0; TriangleIndex != TriangleList.Num(); ++TriangleIndex)
	{
		const FTriangle* Triangle = TriangleList[TriangleIndex];
		if (Triangle->Vertices[0]->SampleIndex == SampleIndex0 && Triangle->Vertices[1]->SampleIndex == SampleIndex1)
		{
			if (VertexIndex)
			{
				*VertexIndex = 0;
			}
			return TriangleIndex;
		}
		if (Triangle->Vertices[1]->SampleIndex == SampleIndex0 && Triangle->Vertices[2]->SampleIndex == SampleIndex1)
		{
			if (VertexIndex)
			{
				*VertexIndex = 1;
			}
			return TriangleIndex;
		}
		if (Triangle->Vertices[2]->SampleIndex == SampleIndex0 && Triangle->Vertices[0]->SampleIndex == SampleIndex1)
		{
			if (VertexIndex)
			{
				*VertexIndex = 2;
			}
			return TriangleIndex;
		}
	}
	// Perfectly normal to get here - when looking for a triangle that is off the outside edge of
	// the graph.
	return INDEX_NONE;
}

TArray<struct FBlendSpaceTriangle> FDelaunayTriangleGenerator::CalculateTriangles() const
{
	TArray<struct FBlendSpaceTriangle> Triangles;
	Triangles.Reserve(TriangleList.Num());
	for (int32 TriangleIndex = 0; TriangleIndex != TriangleList.Num(); ++TriangleIndex)
	{
		const FTriangle* Triangle = TriangleList[TriangleIndex];

		FBlendSpaceTriangle NewTriangle;
		
		FVector2D Vertices[3];
		Vertices[0] = (Triangle->Vertices[0]->Position - GridMin) * RecipGridSize;
		Vertices[1] = (Triangle->Vertices[1]->Position - GridMin) * RecipGridSize;
		Vertices[2] = (Triangle->Vertices[2]->Position - GridMin) * RecipGridSize;

		NewTriangle.Vertices[0] = Vertices[0];
		NewTriangle.Vertices[1] = Vertices[1];
		NewTriangle.Vertices[2] = Vertices[2];
		NewTriangle.SampleIndices[0] = Triangle->Vertices[0]->SampleIndex;
		NewTriangle.SampleIndices[1] = Triangle->Vertices[1]->SampleIndex;
		NewTriangle.SampleIndices[2] = Triangle->Vertices[2]->SampleIndex;

		for (int32 Index = 0; Index != FBlendSpaceTriangle::NUM_VERTICES; ++Index)
		{
			FBlendSpaceTriangleEdgeInfo& EdgeInfo = NewTriangle.EdgeInfo[Index];
			int32 IndexNext = (Index + 1) % FBlendSpaceTriangle::NUM_VERTICES;

			EdgeInfo.NeighbourTriangleIndex = FindTriangleIndexWithEdge(
				Triangle->Vertices[IndexNext]->SampleIndex, Triangle->Vertices[Index]->SampleIndex);
			FVector2D EdgeDir = Vertices[IndexNext] - Vertices[Index];

			// Triangles are wound anticlockwise as viewed from above - rotate the edge 90 deg clockwise
			// to make it be the outwards pointing normal.
			EdgeInfo.Normal = FVector2D(EdgeDir.Y, -EdgeDir.X).GetSafeNormal();

			// Update these if necessary when all triangles have been processed
			EdgeInfo.AdjacentPerimeterTriangleIndices[0] = INDEX_NONE;
			EdgeInfo.AdjacentPerimeterTriangleIndices[1] = INDEX_NONE;
			EdgeInfo.AdjacentPerimeterVertexIndices[0] = INDEX_NONE;
			EdgeInfo.AdjacentPerimeterVertexIndices[1] = INDEX_NONE;
		}
		Triangles.Add(NewTriangle);
	}

	for (int32 TriangleIndex = 0; TriangleIndex != Triangles.Num(); ++TriangleIndex)
	{
		FBlendSpaceTriangle& Triangle = Triangles[TriangleIndex];
		for (int32 Index = 0; Index != FBlendSpaceTriangle::NUM_VERTICES; ++Index)
		{
			FBlendSpaceTriangleEdgeInfo& EdgeInfo = Triangle.EdgeInfo[Index];
			if (EdgeInfo.NeighbourTriangleIndex == INDEX_NONE)
			{
				int32 IndexNext = (Index + 1) % FBlendSpaceTriangle::NUM_VERTICES;

				int32 SampleIndex = Triangle.SampleIndices[Index];
				int32 SampleIndexNext = Triangle.SampleIndices[IndexNext];

				// Update the triangle info to allow traversal around the perimeter of the
				// triangulated region. Iterate through all the other triangles...
				for (int32 OtherTriangleIndex = 0; OtherTriangleIndex != Triangles.Num(); ++OtherTriangleIndex)
				{
					if (OtherTriangleIndex == TriangleIndex)
						continue;

					const FBlendSpaceTriangle& OtherTriangle = Triangles[OtherTriangleIndex];
					// ... then check for a vertex that matches the edge we're considering
					for (int32 VertexIndex = 0; VertexIndex != FBlendSpaceTriangle::NUM_VERTICES; ++VertexIndex)
					{
						int32 VertexIndexPrev = (VertexIndex + FBlendSpaceTriangle::NUM_VERTICES - 1) % FBlendSpaceTriangle::NUM_VERTICES;
						if (OtherTriangle.SampleIndices[VertexIndex] == SampleIndex
							&& OtherTriangle.EdgeInfo[VertexIndexPrev].NeighbourTriangleIndex == INDEX_NONE)
						{
							// We found a perimeter triangle that comes before our edge
							EdgeInfo.AdjacentPerimeterTriangleIndices[0] = OtherTriangleIndex;
							EdgeInfo.AdjacentPerimeterVertexIndices[0] = VertexIndexPrev;
						}
						if (OtherTriangle.SampleIndices[VertexIndex] == SampleIndexNext
							&& OtherTriangle.EdgeInfo[VertexIndex].NeighbourTriangleIndex == INDEX_NONE)
						{
							// We found a perimeter triangle that comes after our edge
							EdgeInfo.AdjacentPerimeterTriangleIndices[1] = OtherTriangleIndex;
							EdgeInfo.AdjacentPerimeterVertexIndices[1] = VertexIndex ;
						}
					}
				}
			}
		}
	}
	return Triangles;
}

bool FDelaunayTriangleGenerator::IsCollinear(const FVertex* A, const FVertex* B, const FVertex* C)
{
	const FVector2D Diff1 = B->Position - A->Position;
	const FVector2D Diff2 = C->Position - A->Position;
	double Cross = Diff1 ^ Diff2;
	return (Cross == 0.f);
}

bool FDelaunayTriangleGenerator::AllCoincident(const TArray<FVertex>& InPoints)
{
	if (InPoints.Num() > 0)
	{
		const FVertex& FirstPoint = InPoints[0];
		for (int32 PointIndex = 0; PointIndex < InPoints.Num(); ++PointIndex)
		{
			const FVertex& Point = InPoints[PointIndex];
			if (Point.Position != FirstPoint.Position)
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

bool FDelaunayTriangleGenerator::FlipTriangles(const int32 TriangleIndexOne, const int32 TriangleIndexTwo)
{
	const FTriangle* A = TriangleList[TriangleIndexOne];
	const FTriangle* B = TriangleList[TriangleIndexTwo];

	// if already optimized, don't have to do any
	FVertex* TestPt = A->FindNonSharingPoint(B);

	// If it's not inside, we don't have to do any
	if (GetCircumcircleState(A, *TestPt) != ECCS_Inside)
	{
		return false;
	}

	FTriangle NewTriangles[2];
	int32 TrianglesMade = 0;

	for (int32 VertexIndexOne = 0; VertexIndexOne < 2; ++VertexIndexOne)
	{
		for (int32 VertexIndexTwo = VertexIndexOne + 1; VertexIndexTwo < 3; ++VertexIndexTwo)
		{
			// Check if these vertices form a valid triangle (should be non-colinear)
			if (IsEligibleForTriangulation(A->Vertices[VertexIndexOne], A->Vertices[VertexIndexTwo], TestPt))
			{
				// Create the new triangle and check if the final (original) vertex falls inside or outside of it's circumcircle
				const FTriangle NewTriangle(A->Vertices[VertexIndexOne], A->Vertices[VertexIndexTwo], TestPt);
				const int32 VertexIndexThree = 3 - (VertexIndexTwo + VertexIndexOne);
				if (GetCircumcircleState(&NewTriangle, *A->Vertices[VertexIndexThree]) == ECCS_Outside)
				{
					// If so store the triangle and increment the number of triangles
					checkf(TrianglesMade < 2, TEXT("Incorrect number of triangles created"));
					NewTriangles[TrianglesMade] = NewTriangle;
					++TrianglesMade;
				}
			}
		}
	}
	
	// In case two triangles were generated the flip was successful so we can add them to the list
	if (TrianglesMade == 2)
	{
		AddTriangle(NewTriangles[0], false);
		AddTriangle(NewTriangles[1], false);
	}

	return TrianglesMade == 2;
}

void FDelaunayTriangleGenerator::AddTriangle(FTriangle& newTriangle, bool bCheckHalfEdge/*=true*/)
{
	// see if it's same vertices
	for (int32 I=0;I<TriangleList.Num(); ++I)
	{
		if (newTriangle == *TriangleList[I])
		{
			return;
		}

		if (bCheckHalfEdge && newTriangle.HasSameHalfEdge(TriangleList[I]))
		{
			return;
		}
	}

	TriangleList.Add(new FTriangle(newTriangle));
}

int32 FDelaunayTriangleGenerator::GenerateTriangles(TArray<FVertex>& PointList, const int32 TotalNum)
{
	if (TotalNum == BLENDSPACE_MINSAMPLE)
	{
		if (IsEligibleForTriangulation(&PointList[0], &PointList[1], &PointList[2]))
		{
			FTriangle Triangle(&PointList[0], &PointList[1], &PointList[2]);
			AddTriangle(Triangle);
		}
	}
	else if (TriangleList.Num() == 0)
	{
		FVertex * TestPoint = &PointList[TotalNum-1];

		// so far no triangle is made, try to make it with new points that are just entered
		for (int32 I=0; I<TotalNum-2; ++I)
		{
			if (IsEligibleForTriangulation(&PointList[I], &PointList[I+1], TestPoint))
			{
				FTriangle NewTriangle (&PointList[I], &PointList[I+1], TestPoint);
				AddTriangle(NewTriangle);
			}
		}
	}
	else
	{
		// get the last addition
		FVertex * TestPoint = &PointList[TotalNum-1];
		int32 TriangleNum = TriangleList.Num();
	
		for (int32 I=0; I<TriangleList.Num(); ++I)
		{
			FTriangle * Triangle = TriangleList[I];
			if (IsEligibleForTriangulation(Triangle->Vertices[0], Triangle->Vertices[1], TestPoint))
			{
				FTriangle NewTriangle (Triangle->Vertices[0], Triangle->Vertices[1], TestPoint);
				AddTriangle(NewTriangle);
			}

			if (IsEligibleForTriangulation(Triangle->Vertices[0], Triangle->Vertices[2], TestPoint))
			{
				FTriangle NewTriangle (Triangle->Vertices[0], Triangle->Vertices[2], TestPoint);
				AddTriangle(NewTriangle);
			}

			if (IsEligibleForTriangulation(Triangle->Vertices[1], Triangle->Vertices[2], TestPoint))
			{
				FTriangle NewTriangle (Triangle->Vertices[1], Triangle->Vertices[2], TestPoint);
				AddTriangle(NewTriangle);
			}
		}

		// this is locally optimization part
		// we need to make sure all triangles are locally optimized. If not optimize it. 
		for (int32 I=0; I<TriangleList.Num(); ++I)
		{
			FTriangle * A = TriangleList[I];
			for (int32 J=I+1; J<TriangleList.Num(); ++J)
			{
				FTriangle * B = TriangleList[J];

				// does share same edge
				if (A->DoesShareSameEdge(B))
				{
					// then test to see if locally optimized
					if (FlipTriangles(I, J))
					{
						// if this flips, remove current triangle
						delete TriangleList[I];
						delete TriangleList[J];
						//I need to remove J first because other wise, 
						//  index J isn't valid anymore
						TriangleList.RemoveAt(J);
						TriangleList.RemoveAt(I);
						// start over since we modified triangle
						// once we don't have any more to flip, we're good to go!
						I=-1;
						break;
					}
				}
			}
		}
	}

	return TriangleList.Num();
}

static FVector GetBaryCentric2D(const FVector2D& Point, const FVector2D& A, const FVector2D& B, const FVector2D& C)
{
	double a = ((B.Y - C.Y) * (Point.X - C.X) + (C.X - B.X) * (Point.Y - C.Y)) / ((B.Y - C.Y) * (A.X - C.X) + (C.X - B.X) * (A.Y - C.Y));
	double b = ((C.Y - A.Y) * (Point.X - C.X) + (A.X - C.X) * (Point.Y - C.Y)) / ((B.Y - C.Y) * (A.X - C.X) + (C.X - B.X) * (A.Y - C.Y));

	return FVector(a, b, 1.0 - a - b);
}


bool FBlendSpaceGrid::FindTriangleThisPointBelongsTo(const FVector2D& TestPoint, FVector& OutBarycentricCoords, FTriangle*& OutTriangle, const TArray<FTriangle*>& TriangleList) const
{
	// Calculate distance from point to triangle and sort the triangle list accordingly
	TArray<FIndexAndDistance> SortedTriangles;
	SortedTriangles.AddUninitialized(TriangleList.Num());
	for (int32 TriangleIndex=0; TriangleIndex<TriangleList.Num(); ++TriangleIndex)
	{
		SortedTriangles[TriangleIndex].Index = TriangleIndex;
		SortedTriangles[TriangleIndex].Distance = TriangleList[TriangleIndex]->GetDistance(TestPoint);
	}
	SortedTriangles.Sort([](const FIndexAndDistance &A, const FIndexAndDistance &B) { return A.Distance < B.Distance; });

	// Now loop over the sorted triangles and test the barycentric coordinates with the point
	for (const FIndexAndDistance& SortedTriangle : SortedTriangles)
	{
		FTriangle* Triangle = TriangleList[SortedTriangle.Index];

		FVector Coords = GetBaryCentric2D(TestPoint, Triangle->Vertices[0]->Position, Triangle->Vertices[1]->Position, Triangle->Vertices[2]->Position);

		// Z coords often has precision error because it's derived from 1-A-B, so do more precise check
		if (FMath::Abs(Coords.Z) < UE_KINDA_SMALL_NUMBER)
		{
			Coords.Z = 0.f;
		}

		// Is the point inside of the triangle, or on it's edge (Z coordinate should always match since the blend samples are set in 2D)
		if ( 0.f <= Coords.X && Coords.X <= 1.0 && 0.f <= Coords.Y && Coords.Y <= 1.0 && 0.f <= Coords.Z && Coords.Z <= 1.0 )
		{
			OutBarycentricCoords = Coords;
			OutTriangle = Triangle;
			return true;
		}
	}

	return false;
}

static FVector2D ClosestPointOnSegment2D(const FVector2D& Point, const FVector2D& StartPoint, const FVector2D& EndPoint, double& T)
{
	const FVector2D Segment = EndPoint - StartPoint;
	const FVector2D VectToPoint = Point - StartPoint;

	// See if closest point is before StartPoint
	const double Dot1 = VectToPoint | Segment;
	if (Dot1 <= 0)
	{
		T = 0.0;
		return StartPoint;
	}

	// See if closest point is beyond EndPoint
	const double Dot2 = Segment | Segment;
	if (Dot2 <= Dot1)
	{
		T = 1.0;
		return EndPoint;
	}

	// Closest Point is within segment
	T = Dot1 / Dot2;
	return StartPoint + Segment * T;
}


void FBlendSpaceGrid::GenerateGridElements(const TArray<FVertex>& SamplePoints, const TArray<FTriangle*>& TriangleList)
{
	check (NumGridDivisions.X > 0 && NumGridDivisions.Y > 0 );
	check (GridMax.ComponentwiseAllGreaterThan(GridMin));

	const int32 TotalNumGridPoints = NumGridPointsForAxis.X * NumGridPointsForAxis.Y;

	GridPoints.Empty(TotalNumGridPoints);

	if (SamplePoints.Num() == 0 || TriangleList.Num() == 0)
	{
		return;
	}

	GridPoints.AddDefaulted(TotalNumGridPoints);

	FVector2D GridPointPosition;		
	for (int32 GridPositionX = 0; GridPositionX < NumGridPointsForAxis.X; ++GridPositionX)
	{
		for (int32 GridPositionY = 0; GridPositionY < NumGridPointsForAxis.Y; ++GridPositionY)
		{
			FTriangle * SelectedTriangle = NULL;
			FEditorElement& GridPoint = GetElement(GridPositionX, GridPositionY);

			GridPointPosition = GetPosFromIndex(GridPositionX, GridPositionY);

			FVector Weights;
			if ( FindTriangleThisPointBelongsTo(GridPointPosition, Weights, SelectedTriangle, TriangleList) )
			{
				// found it
				GridPoint.Weights[0] = Weights.X;
				GridPoint.Weights[1] = Weights.Y;
				GridPoint.Weights[2] = Weights.Z;
				GridPoint.Indices[0] = SelectedTriangle->Vertices[0]->SampleIndex;
				GridPoint.Indices[1] = SelectedTriangle->Vertices[1]->SampleIndex;
				GridPoint.Indices[2] = SelectedTriangle->Vertices[2]->SampleIndex;
				check(GridPoint.Indices[0] != INDEX_NONE);
				check(GridPoint.Indices[1] != INDEX_NONE);
				check(GridPoint.Indices[2] != INDEX_NONE);
			}
			else
			{
				// Work through all the edges and find the one with a point closest to this grid position.
				int32 ClosestTriangleIndex = 0;
				int32 ClosestEdgeIndex = 0;
				double ClosestDistance = UE_DOUBLE_BIG_NUMBER;
				double ClosestT = 0.0;

				// Just walk through all the edges from all the triangles and find the closest ones
				for (int32 TriangleIndex = 0; TriangleIndex < TriangleList.Num(); ++TriangleIndex)
				{
					const FTriangle* Triangle = TriangleList[TriangleIndex];
					for (int32 EdgeIndex = 0 ; EdgeIndex != 3 ; ++EdgeIndex)
					{
						double T;
						const FVector2D ClosestPoint = ClosestPointOnSegment2D(
							GridPointPosition,
							Triangle->Edges[EdgeIndex].Vertices[0]->Position,
							Triangle->Edges[EdgeIndex].Vertices[1]->Position,
							T);
						const double Distance = (ClosestPoint - GridPointPosition).SizeSquared();

						if (Distance < ClosestDistance)
						{
							ClosestTriangleIndex = TriangleIndex;
							ClosestEdgeIndex = EdgeIndex;
							ClosestDistance = Distance;
							ClosestT = T;
						}
					}
				}

				const FTriangle* Triangle = TriangleList[ClosestTriangleIndex];

				GridPoint.Weights[0] = 1.0 - ClosestT;
				GridPoint.Indices[0] = Triangle->Edges[ClosestEdgeIndex].Vertices[0]->SampleIndex;
				GridPoint.Weights[1] = ClosestT;
				GridPoint.Indices[1] = Triangle->Edges[ClosestEdgeIndex].Vertices[1]->SampleIndex;
			}
		}
	}
}

/** 
* Convert grid index (GridX, GridY) to triangle coords and returns FVector2D
*/
const FVector2D FBlendSpaceGrid::GetPosFromIndex(const int32 GridX, const int32 GridY) const
{
	// grid X starts from 0 -> N when N == GridSizeX
	// grid Y starts from 0 -> N when N == GridSizeY
	// LeftBottom will map to Grid 0, 0
	// RightTop will map to Grid N, N

	FVector2D CoordDim = GridMax - GridMin;
	FVector2D EachGridSize = CoordDim / NumGridDivisions;

	// for now only 2D
	return FVector2D(GridX*EachGridSize.X+GridMin.X, GridY*EachGridSize.Y+GridMin.Y);
}

const FEditorElement& FBlendSpaceGrid::GetElement(const int32 GridX, const int32 GridY) const
{
	check (NumGridPointsForAxis.X >= GridX);
	check (NumGridPointsForAxis.Y >= GridY);

	check (GridPoints.Num() > 0 );
	return GridPoints[GridY * NumGridPointsForAxis.X + GridX];
}

FEditorElement& FBlendSpaceGrid::GetElement(const int32 GridX, const int32 GridY)
{
	FEditorElement& Test = const_cast<FEditorElement &>(std::as_const(*this).GetElement(GridX, GridY));

	check (NumGridPointsForAxis.X >= GridX);
	check (NumGridPointsForAxis.Y >= GridY);

	check (GridPoints.Num() > 0 );
	FEditorElement& Test2 = GridPoints[GridY * NumGridPointsForAxis.X + GridX];
	return Test2;
}

#undef LOCTEXT_NAMESPACE
