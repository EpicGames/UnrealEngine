// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/BlendSpace.h"

struct FTriangle;

//////////////////////////////////////////////////////////////////////////
// SAnimationBlendSpace

struct FTriangle;

/**
 * Utility class for FDelaunayTriangleGenerator 
 * This represents Points in FDelaunayTriangleGenerator
 */
struct FVertex
{
	// position of Point
	FVector2D Position;

	// Triangles this point belongs to
	TArray<FTriangle *> Triangles;

	// The original animation sample associated with this point
	int32 SampleIndex;
	
	FVertex(FVector2D InPosition, int32 InSampleIndex) : Position(InPosition), SampleIndex(InSampleIndex) {}

	bool operator==(const FVector2D& Other) const
	{
		// if same position, it's same point
		return (Other == Position);
	}

	bool operator==( const FVertex& Other ) const
	{
		// if same position, it's same point
		return (Other.Position == Position);
	}

	void AddTriangle(FTriangle * NewTriangle)
	{
		Triangles.AddUnique(NewTriangle);
	}

	void RemoveTriangle(FTriangle * TriangleToRemove)
	{
		Triangles.Remove(TriangleToRemove);
	}

	double GetDistance(const FVertex& Other)
	{
		return (Other.Position-Position).Size();
	}
};

struct FHalfEdge
{
	// 3 vertices in CCW order
	FVertex* Vertices[2]; 

	FHalfEdge(){};
	FHalfEdge(FVertex * A, FVertex * B)
	{
		Vertices[0] = A;
		Vertices[1] = B;
	}

	bool DoesShare(const FHalfEdge& A) const
	{
		return (Vertices[0] == A.Vertices[1] && Vertices[1] == A.Vertices[0]);
	}

	bool operator==( const FHalfEdge& Other ) const 
	{
		// if same position, it's same point
		return FMemory::Memcmp(Other.Vertices, Vertices, sizeof(Vertices)) == 0;
	}
};
/**
 * Utility class for FDelaunayTriangleGenerator 
 * This represents Triangle in FDelaunayTriangleGenerator
 */
struct FTriangle
{
	// 3 vertices in CCW order
	FVertex* Vertices[3]; 
	// average points for Vertices
	FVector2D Center;
	// FEdges
	FHalfEdge Edges[3];

	void UpdateCenter() 
	{
		Center = (Vertices[0]->Position + Vertices[1]->Position + Vertices[2]->Position) / 3.0;
	}

	bool operator==( const FTriangle& Other ) const 
	{
		// if same position, it's same point
		return FMemory::Memcmp(Other.Vertices, Vertices, sizeof(Vertices)) == 0;
	}

	FTriangle(FTriangle & Copy)
	{
		FMemory::Memcpy(Vertices, Copy.Vertices);
		FMemory::Memcpy(Edges, Copy.Edges);
		UpdateCenter();

		Vertices[0]->AddTriangle(this);
		Vertices[1]->AddTriangle(this);
		Vertices[2]->AddTriangle(this);
	}

	FTriangle(FVertex * A, FVertex * B, FVertex * C)
	{
		Vertices[0] = A;
		Vertices[1] = B;
		Vertices[2] = C;
		UpdateCenter();

		Vertices[0]->AddTriangle(this);
		Vertices[1]->AddTriangle(this);
		Vertices[2]->AddTriangle(this);
		// when you make triangle first time, make sure it stays in CCW
		MakeCCW();

		// now create edges, this should be in the CCW order
		Edges[0] = FHalfEdge(Vertices[0], Vertices[1]);
		Edges[1] = FHalfEdge(Vertices[1], Vertices[2]);
		Edges[2] = FHalfEdge(Vertices[2], Vertices[0]);
	}

	FTriangle(FVertex * A)
	{
		Vertices[0] = A;
		Vertices[1] = A;
		Vertices[2] = A;
		UpdateCenter();

		Vertices[0]->AddTriangle(this);
		Vertices[1]->AddTriangle(this);
		Vertices[2]->AddTriangle(this);

		// now create edges, this should be in the CCW order
		Edges[0] = FHalfEdge(Vertices[0], Vertices[1]);
		Edges[1] = FHalfEdge(Vertices[1], Vertices[2]);
		Edges[2] = FHalfEdge(Vertices[2], Vertices[0]);
	}

	FTriangle(FVertex * A, FVertex * B)
	{
		Vertices[0] = A;
		Vertices[1] = B;
		Vertices[2] = B;
		UpdateCenter();

		Vertices[0]->AddTriangle(this);
		Vertices[1]->AddTriangle(this);
		Vertices[2]->AddTriangle(this);

		// now create edges, this should be in the CCW order
		Edges[0] = FHalfEdge(Vertices[0], Vertices[1]);
		Edges[1] = FHalfEdge(Vertices[1], Vertices[2]);
		Edges[2] = FHalfEdge(Vertices[2], Vertices[0]);
	}
	
	FTriangle()
	{
		Vertices[0] = nullptr;
		Vertices[1] = nullptr;
		Vertices[2] = nullptr;
	}

	~FTriangle()
	{
		for (int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
		{
			if (Vertices[VertexIndex])
			{
				Vertices[VertexIndex]->RemoveTriangle(this);
			}
		}
	}

	bool Contains (const FVertex& Other) const
	{
		return (Other == *Vertices[0] || Other == *Vertices[1] || Other == *Vertices[2]);
	}

	double GetDistance(const FVertex& Other) const
	{
		return (Other.Position-Center).Size();
	}

	double GetDistance(const FVector2D& Other) const
	{
		return (Other-Center).Size();
	}

	bool HasSameHalfEdge(const FTriangle* Other) const
	{
		for (int32 I=0; I<3; ++I)
		{
			for (int32 J=0; J<3; ++J)
			{
				if (Other->Edges[I] == Edges[J])
				{
					return true;
				}
			}
		}

		return false;
	}

	bool DoesShareSameEdge(const FTriangle* Other) const
	{
		for (int32 I=0; I<3; ++I)
		{
			for (int32 J=0; J<3; ++J)
			{
				if (Other->Edges[I].DoesShare(Edges[J]))
				{
					return true;
				}
			}
		}

		return false;
	}

	// find point that doesn't share with this
	// this should only get called if it shares same edge
	FVertex * FindNonSharingPoint(const FTriangle* Other) const
	{
		if (!Contains(*Other->Vertices[0]))
		{
			return Other->Vertices[0];
		}

		if (!Contains(*Other->Vertices[1]))
		{
			return Other->Vertices[1];
		}

		if (!Contains(*Other->Vertices[2]))
		{
			return Other->Vertices[2];
		}

		return NULL;
	}

private:
	void MakeCCW()
	{
		// this eventually has to happen on the plane that contains this 3 points
		// for now we ignore Z
		FVector2D Diff1 = Vertices[1]->Position-Vertices[0]->Position;
		FVector2D Diff2 = Vertices[2]->Position-Vertices[0]->Position;

		double Result = Diff1.X*Diff2.Y - Diff1.Y*Diff2.X;

		check (Result != 0.f);

		// it's in left side, we need this to be right side
		if (Result < 0.f)
		{
			// swap 1&2 
			FVertex * TempPt = Vertices[2];
			Vertices[2] = Vertices[1];
			Vertices[1] = TempPt;
		}
	}
};

/**
 * Generates triangles from sample point using Delaunay Triangulation, 
 * TriangleList contains the list of triangle after generated.
 */
class FDelaunayTriangleGenerator
{
public:
	enum ECircumCircleState
	{
		ECCS_Outside = -1,
		ECCS_On = 0,
		ECCS_Inside = 1,
	};
	/** 
	 * Reset all data
	 */
	void Reset();
	void EmptyTriangles();
	void EmptySamplePoints();

	/**
	 * Generate triangles from SamplePointList
	 */
	void Triangulate(EPreferredTriangulationDirection PreferredTriangulationDirection);

	/**
	 * Converts from our internal representation to the runtime triangles
	 */
	TArray<struct FBlendSpaceTriangle> CalculateTriangles() const;

	/** 
	 * Add new sample point to SamplePointList and its corresponding sample index in the blendspace
	 * It won't be added if already exists
	 */
	void AddSamplePoint(const FVector2D& NewPoint, const int32 SampleIndex);

	/** 
	 * This is for debug purpose to step only one to triangulate
	 * StartIndex should increase manually
	 */
	void Step(int32 StartIndex);

	~FDelaunayTriangleGenerator()
	{
		Reset();
	}

	/** 
	 * Get TriangleList
	 */
	const TArray<FTriangle*> & GetTriangleList() const { return TriangleList; };

	/** 
	 * Get SamplePointList
	 */
	const TArray<FVertex> & GetSamplePointList() const { return SamplePointList; };

	/* Set the grid box, so we can normalize the sample points */
	void SetGridBox(const FBlendParameter& BlendParamX, const FBlendParameter& BlendParamY);

private:
	/**
	 * Sort SamplePointList in the order of 1) -X->+X 2) -Y->+Y 3) -Z->+Z
	 * Sort Point of the input point in the order of
	 * 1) -> +x. 2) -> +y, 3) -> +z
	 * by that, it will sort from -X to +X, if same is found, next will be Y, and if same, next will be Z
	 */
	void SortSamples();

	/**
	 * Tries to make the split of ambiguous triangles that are aligned with the grid such that they are symmetric.
	 */
	void AdjustEdgeDirections(EPreferredTriangulationDirection PreferredTriangulationDirection);

	/**
	 * Finds the triangle index that contains an edge referencing the samples in that order.
	 * Optionally returns the index of the vertex in the found triangle that is the start of the edge.
	 */
	int32 FindTriangleIndexWithEdge(int32 SampleIndex0, int32 SampleIndex1, int32* VertexIndex = nullptr) const;

	/**
	 * The key function in Delaunay Triangulation
	 * return true if the TestPoint is WITHIN the triangle circumcircle
	 *	http://en.wikipedia.org/wiki/Delaunay_triangulation 
	 */
	ECircumCircleState GetCircumcircleState(const FTriangle* T, const FVertex& TestPoint);

	/**
	 * return true if they can make triangle
	 */
	bool IsEligibleForTriangulation(const FVertex* A, const FVertex* B, const FVertex* C)
	{
		return (IsCollinear(A, B, C)==false);
	}

	/** 
	 * return true if 3 points are collinear
	 * by that if those 3 points create straight line
	 */
	bool IsCollinear(const FVertex* A, const FVertex* B, const FVertex* C);

	/** 
	 * return true if all points are coincident
	 * (i.e. if all points are the same)
	 */
	bool AllCoincident(const TArray<FVertex>& InPoints);

	/**
	 * Flip TriangleList(I) with TriangleList(J). 
	 */
	bool FlipTriangles(const int32 TriangleIndexOne, const int32 TriangleIndexTwo);

	/** 
	 * Add new Triangle
	 */
	void AddTriangle(FTriangle & newTriangle, bool bCheckHalfEdge=true);

	/** 
	 * Used as incremental step to triangulate all points
	 * Create triangles TotalNum of PointList
	 */
	int32 GenerateTriangles(TArray<FVertex> & PointList, const int32 TotalNum);

private:
	/**
	 * Data Structure for triangulation
	 * SamplePointList is the input data
	 */
	TArray<FVertex>		SamplePointList;

	/**
	 * Output data for this generator
	 */
	TArray<FTriangle*>	TriangleList;

	/**
	 * We need to normalize the points before the circumcircle test, so we need the extents of the grid.
	 * Store 1 / GridSize to avoid recalculating it every time through GetCircumcircleState
	*/
	FVector2D				GridMin;
	FVector2D				RecipGridSize;
};

// @todo fixmelh : this code is mess between fvector2D and fvector
// ideally FBlendSpaceGrid will be handled in 3D, and SBlendSpaceGridWidget only knows about 2D
// in the future this should all change to 3D

/**
 * BlendSpace Grid
 * Using triangulated space, create FEditorElement of each grid point using SamplePoint
 */
class FBlendSpaceGrid
{
public:
	/**
	 * Find Triangle this TestPoint is within
	 * 
	 * @param	TestPoint				Point to test
	 * @param	OutBaryCentricCoords	Output BaryCentricCoords2D of the point in the triangle // for now it's only 2D
	 * @param	OutTriangle				The triangle that this point is within or lie
	 * @param	TriangleList			TriangleList to test
	 * 
	 * @return	true if successfully found the triangle this point is within
	 */
	bool FindTriangleThisPointBelongsTo(const FVector2D& TestPoint, FVector& OutBarycentricCoords, FTriangle* & OutTriangle, const TArray<FTriangle*> & TriangleList) const;

	/**  
	 * Fill up Grid GridPoints using TriangleList input - Grid information should have been set by SetGridInfo
	 * 
	 * @param	SamplePoints		: Sample Point List
	 * @param	TriangleList		: List of triangles
	 */
	void GenerateGridElements(const TArray<FVertex>& SamplePoints, const TArray<FTriangle*> & TriangleList);

	/** 
	 * default value 
	 */
	FBlendSpaceGrid()
		: GridMin(0, 0)
		, GridMax(100, 100)
		, NumGridPointsForAxis(5, 5)
	{
	}

	void Reset()
	{
		GridPoints.Empty();
	}

	void SetGridInfo(const FBlendParameter& BlendParamX, const FBlendParameter& BlendParamY)
	{
		NumGridPointsForAxis.X = BlendParamX.GridNum + 1;
		NumGridPointsForAxis.Y = BlendParamY.GridNum + 1;

		NumGridDivisions.X = BlendParamX.GridNum;
		NumGridDivisions.Y = BlendParamY.GridNum;

		GridMin.X = BlendParamX.Min;
		GridMax.X = BlendParamX.Max;

		GridMin.Y = BlendParamY.Min;
		GridMax.Y = BlendParamY.Max;
	}
	
	const FEditorElement& GetElement(const int32 GridX, const int32 GridY) const;
	const TArray<FEditorElement>& GetElements() const { return GridPoints;}
	
	/** 
	 * Convert grid index (GridX, GridY) to triangle coords and returns FVector2D
	*/
	const FVector2D GetPosFromIndex(const int32 GridX, const int32 GridY) const;

private:
	FEditorElement& GetElement(const int32 GridX, const int32 GridY);
	// Grid Dimension
	FVector2D GridMin;
	FVector2D GridMax;

	// how many rows/cols for each axis
	FIntPoint NumGridPointsForAxis;
	FIntPoint NumGridDivisions;

	// Each point data -output data
	TArray<FEditorElement> GridPoints; // 2D array saved in 1D array, to search (x, y), x*GridSizeX+y;
};
