// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/BlendSpaceUtilities.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimSequence.h"

int32 FSyncPattern::IndexOf(FName Name, int32 StartIndex /*= 0*/) const
{
	for (int Ind = StartIndex; Ind < MarkerNames.Num(); ++Ind)
	{
		if (MarkerNames[Ind] == Name)
		{
			return Ind;
		}
	}
	return INDEX_NONE;
}

bool FSyncPattern::DoOneMatch(const TArray<FName>& TestMarkerNames, int32 StartIndex)
{
	int32 MyMarkerIndex = StartIndex;
	int32 TestMarkerIndex = 0;

	do
	{
		if (MarkerNames[MyMarkerIndex] != TestMarkerNames[TestMarkerIndex])
		{
			return false;
		}
		MyMarkerIndex = (MyMarkerIndex + 1) % MarkerNames.Num();
		TestMarkerIndex = (TestMarkerIndex + 1) % TestMarkerNames.Num();
	} while (MyMarkerIndex != StartIndex || TestMarkerIndex != 0); // Did we get back to the start without failing
	return true;
}

bool FSyncPattern::DoesPatternMatch(const TArray<FName>& TestMarkerNames)
{
	check(TestMarkerNames.Num() > 0 && MarkerNames.Num() > 0);

	FName StartMarker = TestMarkerNames[0];

	int32 StartIndex = IndexOf(StartMarker);
	while (StartIndex != INDEX_NONE)
	{
		if (DoOneMatch(TestMarkerNames, StartIndex))
		{
			return true;
		}
		StartIndex = IndexOf(StartMarker, StartIndex + 1);
	}
	return false;
}

int32 FBlendSpaceUtilities::GetHighestWeightSample(const TArray<struct FBlendSampleData> &SampleDataList)
{
	int32 HighestWeightIndex = 0;
	float HighestWeight = SampleDataList[HighestWeightIndex].GetClampedWeight();
	for (int32 I = 1; I < SampleDataList.Num(); I++)
	{
		if (SampleDataList[I].GetClampedWeight() > HighestWeight)
		{
			HighestWeightIndex = I;
			HighestWeight = SampleDataList[I].GetClampedWeight();
		}
	}
	return HighestWeightIndex;
}

int32 FBlendSpaceUtilities::GetHighestWeightMarkerSyncSample(const TArray<struct FBlendSampleData> &SampleDataList, const TArray<struct FBlendSample>& BlendSamples)
{
	int32 HighestWeightIndex = INDEX_NONE;
	float HighestWeight = FLT_MIN;

	for (int32 I = 0; I < SampleDataList.Num(); I++)
	{
		const FBlendSampleData& SampleData = SampleDataList[I];
		if (SampleData.GetClampedWeight() > HighestWeight &&
			(BlendSamples[SampleData.SampleDataIndex].Animation != nullptr &&
			 BlendSamples[SampleData.SampleDataIndex].Animation->AuthoredSyncMarkers.Num() > 0))
		{
			HighestWeightIndex = I;
			HighestWeight = SampleData.GetClampedWeight();
		}
	}
	return HighestWeightIndex;
}
