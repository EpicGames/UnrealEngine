// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FSyncPattern
{
	// The markers that make up this pattern
	TArray<FName> MarkerNames;

	// Returns the index of the supplied name in the array of marker names
	// Search starts at StartIndex
	int32 IndexOf(const FName Name, const int32 StartIndex = 0) const;
	
	// Tests the supplied pattern against ours, starting at the supplied start index
	bool DoOneMatch(const TArray<FName>& TestMarkerNames, const int32 StartIndex);

	// Testes the supplied pattern against ourselves. Is not a straight forward array match
	// (for example a,b,c,a would match b,c,a,a)
	bool DoesPatternMatch(const TArray<FName>& TestMarkerNames);
};

class FBlendSpaceUtilities
{
public:
	static int32 GetHighestWeightSample(const TArray<struct FBlendSampleData> &SampleDataList);
	static int32 GetHighestWeightMarkerSyncSample(const TArray<struct FBlendSampleData> &SampleDataList, const TArray<struct FBlendSample>& BlendSamples);
};
