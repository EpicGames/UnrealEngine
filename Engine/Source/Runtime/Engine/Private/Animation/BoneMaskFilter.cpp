// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	Bone Mask Filter data holder class implementation
 *
 */

#include "Animation/AnimData/BoneMaskFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoneMaskFilter)

 /////////////////////////////////////////////////////
 // UBoneMaskFilter 

UBoneMaskFilter::UBoneMaskFilter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

