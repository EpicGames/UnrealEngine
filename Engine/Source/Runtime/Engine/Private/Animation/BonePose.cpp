// Copyright Epic Games, Inc. All Rights Reserved.

#include "BonePose.h"
#include "Animation/AnimCurveTypes.h"
#include "AnimationRuntime.h"
#include "AnimEncoding.h"
#include "Animation/AnimSequenceHelpers.h"
#include "AutoRTFM.h"
#include "HAL/IConsoleManager.h"

#if INTEL_ISPC
#include "BonePose.ispc.generated.h"

static_assert(sizeof(ispc::FTransform) == sizeof(FTransform), "sizeof(ispc::FTransform) != sizeof(FTransform)");
#endif

#if !defined(ANIM_BONE_POSE_ISPC_ENABLED_DEFAULT)
#define ANIM_BONE_POSE_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bAnim_BonePose_ISPC_Enabled = INTEL_ISPC && ANIM_BONE_POSE_ISPC_ENABLED_DEFAULT;
#else
static bool bAnim_BonePose_ISPC_Enabled = ANIM_BONE_POSE_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarAnimBonePoseISPCEnabled(TEXT("a.BonePose.ISPC"), bAnim_BonePose_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in bone pose calculations"));
#endif

// Normalizes all rotations in this pose
void FCompactPose::NormalizeRotations()
{
	if (bAnim_BonePose_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				this->Bones.GetData(),
				this->Bones.Num() * sizeof(FTransform));
			ispc::NormalizeRotations(
				reinterpret_cast<ispc::FTransform*>(this->Bones.GetData()), 
				this->Bones.Num());
		};
#endif
	}
	else
	{
		for (FTransform& Bone : this->Bones)
		{
			Bone.NormalizeRotation();
		}
	}
}

// Sets every bone transform to Identity
void FCompactPose::ResetToAdditiveIdentity()
{
	if (bAnim_BonePose_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				this->Bones.GetData(),
				this->Bones.Num() * sizeof(FTransform));
			ispc::ResetToAdditiveIdentity(
				reinterpret_cast<ispc::FTransform*>(this->Bones.GetData()),
				this->Bones.Num());
		};
#endif
	}
	else
	{
		for (FTransform& Bone : this->Bones)
		{
			Bone.SetIdentityZeroScale();
		}
	}
}

// Normalizes all rotations in this pose
void FCompactHeapPose::NormalizeRotations()
{
	if (bAnim_BonePose_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				this->Bones.GetData(),
				this->Bones.Num() * sizeof(FTransform));
			ispc::NormalizeRotations(
				reinterpret_cast<ispc::FTransform*>(this->Bones.GetData()),
				this->Bones.Num());
		};
#endif
	}
	else
	{
		for (FTransform& Bone : this->Bones)
		{
			Bone.NormalizeRotation();
		}
	}
}

// Sets every bone transform to Identity
void FCompactHeapPose::ResetToAdditiveIdentity()
{
	if (bAnim_BonePose_ISPC_Enabled)
	{
#if INTEL_ISPC
		UE_AUTORTFM_OPEN_NO_VALIDATION // Open writes are recorded
		{
			AutoRTFM::RecordOpenWrite(
				this->Bones.GetData(),
				this->Bones.Num() * sizeof(FTransform));
			ispc::ResetToAdditiveIdentity(
				reinterpret_cast<ispc::FTransform*>(this->Bones.GetData()),
				this->Bones.Num());
		};
#endif
	}
	else
	{
		for (FTransform& Bone : this->Bones)
		{
			Bone.SetIdentityZeroScale();
		}
	}
}

void FMeshPose::ResetToRefPose()
{
	FAnimationRuntime::FillWithRefPose(Bones, *BoneContainer);
}

void FMeshPose::ResetToIdentity()
{
	FAnimationRuntime::InitializeTransform(*BoneContainer, Bones);
}


bool FMeshPose::ContainsNaN() const
{
	const TArray<FBoneIndexType> & RequiredBoneIndices = BoneContainer->GetBoneIndicesArray();
	for (int32 Iter = 0; Iter < RequiredBoneIndices.Num(); ++Iter)
	{
		const int32 BoneIndex = RequiredBoneIndices[Iter];
		if (Bones[BoneIndex].ContainsNaN())
		{
			return true;
		}
	}

	return false;
}

bool FMeshPose::IsNormalized() const
{
	const TArray<FBoneIndexType> & RequiredBoneIndices = BoneContainer->GetBoneIndicesArray();
	for (int32 Iter = 0; Iter < RequiredBoneIndices.Num(); ++Iter)
	{
		int32 BoneIndex = RequiredBoneIndices[Iter];
		const FTransform& Trans = Bones[BoneIndex];
		if (!Bones[BoneIndex].IsRotationNormalized())
		{
			return false;
		}
	}

	return true;
}

FTransform ExtractTransformForKey(int32 Key, const FRawAnimSequenceTrack &TrackToExtract)
{
	static const FVector DefaultScale3D = FVector(1.f);
	const bool bHasScaleKey = TrackToExtract.ScaleKeys.Num() > 0;

	const int32 PosKeyIndex = FMath::Min(Key, TrackToExtract.PosKeys.Num() - 1);
	const int32 RotKeyIndex = FMath::Min(Key, TrackToExtract.RotKeys.Num() - 1);
	if (bHasScaleKey)
	{
		const int32 ScaleKeyIndex = FMath::Min(Key, TrackToExtract.ScaleKeys.Num() - 1);
		return FTransform(FQuat(TrackToExtract.RotKeys[RotKeyIndex]), FVector(TrackToExtract.PosKeys[PosKeyIndex]), FVector(TrackToExtract.ScaleKeys[ScaleKeyIndex]));
	}
	else
	{
		return FTransform(FQuat(TrackToExtract.RotKeys[RotKeyIndex]), FVector(TrackToExtract.PosKeys[PosKeyIndex]), FVector(DefaultScale3D));
	}
}

template<bool bInterpolateT>
void BuildPoseFromRawDataInternal(const TArray<FRawAnimSequenceTrack>& InAnimationData, const TArray<struct FTrackToSkeletonMap>& TrackToSkeletonMapTable, FCompactPose& InOutPose, int32 KeyIndex1, int32 KeyIndex2, float Alpha, float TimePerKey, const TMap<int32, const FTransformCurve*>* AdditiveBoneTransformCurves)
{
	const int32 NumTracks = InAnimationData.Num();
	const FBoneContainer& RequiredBones = InOutPose.GetBoneContainer();

	TArray<UE::Anim::Retargeting::FRetargetTracking>& RetargetTracking = UE::Anim::FBuildRawPoseScratchArea::Get().RetargetTracking;
	RetargetTracking.Reset(NumTracks);

	TArray<FVirtualBoneCompactPoseData>& VBCompactPoseData =  UE::Anim::FBuildRawPoseScratchArea::Get().VirtualBoneCompactPoseData;
	VBCompactPoseData = RequiredBones.GetVirtualBoneCompactPoseData();

	FCompactPose Key2Pose;
	if (bInterpolateT)
	{
		Key2Pose.CopyBonesFrom(InOutPose);
	}

	for (int32 TrackIndex = 0; TrackIndex < NumTracks; TrackIndex++)
	{
		const int32 SkeletonBoneIndex = TrackToSkeletonMapTable[TrackIndex].BoneTreeIndex;
		// not sure it's safe to assume that SkeletonBoneIndex can never be INDEX_NONE
		if ((SkeletonBoneIndex != INDEX_NONE) && (SkeletonBoneIndex < MAX_BONES))
		{
			const FCompactPoseBoneIndex PoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonIndex(SkeletonBoneIndex);
			if (PoseBoneIndex != INDEX_NONE)
			{
				for (int32 Idx = 0; Idx < VBCompactPoseData.Num(); ++Idx)
				{
					FVirtualBoneCompactPoseData& VB = VBCompactPoseData[Idx];
					if (PoseBoneIndex == VB.VBIndex)
					{
						// Remove this bone as we have written data for it
						VBCompactPoseData.RemoveAtSwap(Idx, EAllowShrinking::No);
						break; //Modified TArray so must break here
					}
				}
				// extract animation

				const FRawAnimSequenceTrack& TrackToExtract = InAnimationData[TrackIndex];

				const FTransformCurve* const * AdditiveBoneTransformCurve = AdditiveBoneTransformCurves ? AdditiveBoneTransformCurves->Find(SkeletonBoneIndex) : nullptr;

				// Bail out (with rather wacky data) if data is empty for some reason.
				if (TrackToExtract.PosKeys.Num() == 0 || TrackToExtract.RotKeys.Num() == 0)
				{
					InOutPose[PoseBoneIndex].SetIdentity();

					if (bInterpolateT)
					{
						Key2Pose[PoseBoneIndex].SetIdentity();
					}
				}
				else
				{
					InOutPose[PoseBoneIndex] = ExtractTransformForKey(KeyIndex1, TrackToExtract);

					if (bInterpolateT)
					{
						Key2Pose[PoseBoneIndex] = ExtractTransformForKey(KeyIndex2, TrackToExtract);
					}


					if (AdditiveBoneTransformCurve)
					{
						const FTransform PoseOneAdditive = (*AdditiveBoneTransformCurve)->Evaluate(KeyIndex1 * TimePerKey, 1.f);
						const FTransform PoseOneLocalTransform = InOutPose[PoseBoneIndex];
						InOutPose[PoseBoneIndex].SetRotation(PoseOneLocalTransform.GetRotation() * PoseOneAdditive.GetRotation());
						InOutPose[PoseBoneIndex].SetTranslation(PoseOneLocalTransform.TransformPosition(PoseOneAdditive.GetTranslation()));
						InOutPose[PoseBoneIndex].SetScale3D(PoseOneLocalTransform.GetScale3D() * PoseOneAdditive.GetScale3D());

						if (bInterpolateT)
						{
							const FTransform PoseTwoAdditive = (*AdditiveBoneTransformCurve)->Evaluate(KeyIndex2 * TimePerKey, 1.f);
							const FTransform PoseTwoLocalTransform = Key2Pose[PoseBoneIndex];
							Key2Pose[PoseBoneIndex].SetRotation(PoseTwoLocalTransform.GetRotation() * PoseTwoAdditive.GetRotation());
							Key2Pose[PoseBoneIndex].SetTranslation(PoseTwoLocalTransform.TransformPosition(PoseTwoAdditive.GetTranslation()));
							Key2Pose[PoseBoneIndex].SetScale3D(PoseTwoLocalTransform.GetScale3D() * PoseTwoAdditive.GetScale3D());
						}
					}
				}

				RetargetTracking.Add(UE::Anim::Retargeting::FRetargetTracking(PoseBoneIndex, SkeletonBoneIndex));
			}
		}
	}

	//Build Virtual Bones
	if (VBCompactPoseData.Num() > 0)
	{
		FCSPose<FCompactPose> CSPose1;
		CSPose1.InitPose(InOutPose);

		FCSPose<FCompactPose> CSPose2;
		if (bInterpolateT)
		{
			CSPose2.InitPose(Key2Pose);
		}

		for (FVirtualBoneCompactPoseData& VB : VBCompactPoseData)
		{
			FTransform Source = CSPose1.GetComponentSpaceTransform(VB.SourceIndex);
			FTransform Target = CSPose1.GetComponentSpaceTransform(VB.TargetIndex);
			InOutPose[VB.VBIndex] = Target.GetRelativeTransform(Source);

			if (bInterpolateT)
			{
				FTransform Source2 = CSPose2.GetComponentSpaceTransform(VB.SourceIndex);
				FTransform Target2 = CSPose2.GetComponentSpaceTransform(VB.TargetIndex);
				Key2Pose[VB.VBIndex] = Target2.GetRelativeTransform(Source2);
			}
		}
	}

	if (bInterpolateT)
	{
		for (FCompactPoseBoneIndex BoneIndex : InOutPose.ForEachBoneIndex())
		{
			InOutPose[BoneIndex].Blend(InOutPose[BoneIndex], Key2Pose[BoneIndex], Alpha);
		}
	}
}

void BuildPoseFromRawData(
	const TArray<FRawAnimSequenceTrack>& InAnimationData, 
	const TArray<struct FTrackToSkeletonMap>& TrackToSkeletonMapTable, 
	FCompactPose& InOutPose, 
	float InTime, 
	EAnimInterpolationType Interpolation, 
	int32 NumFrames, 
	float SequenceLength, 
	FName RetargetSource, 
	const TMap<int32, const FTransformCurve*>* AdditiveBoneTransformCurves /*= nullptr*/
	)
{
	USkeleton* MySkeleton = InOutPose.GetBoneContainer().GetSkeletonAsset();
	if (MySkeleton)
	{
		const TArray<FTransform>& RetargetTransforms = MySkeleton->GetRefLocalPoses(RetargetSource);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BuildPoseFromRawData(InAnimationData, TrackToSkeletonMapTable, InOutPose, InTime, Interpolation, NumFrames, SequenceLength, RetargetSource, RetargetTransforms, AdditiveBoneTransformCurves);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void BuildPoseFromRawData(
	const TArray<FRawAnimSequenceTrack>& InAnimationData, 
	const TArray<struct FTrackToSkeletonMap>& TrackToSkeletonMapTable, 
	FCompactPose& InOutPose, 
	float InTime, 
	EAnimInterpolationType Interpolation, 
	int32 NumFrames, 
	float SequenceLength, 
	FName SourceName, 
	const TArray<FTransform>& RetargetTransforms,
	const TMap<int32, const FTransformCurve*>* AdditiveBoneTransformCurves /*= nullptr*/
	)
{
	int32 KeyIndex1, KeyIndex2;
	float Alpha;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAnimationRuntime::GetKeyIndicesFromTime(KeyIndex1, KeyIndex2, Alpha, InTime, NumFrames, SequenceLength);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (Interpolation == EAnimInterpolationType::Step)
	{
		Alpha = 0.f;
	}

	bool bInterpolate = true;

	if (Alpha < UE_KINDA_SMALL_NUMBER)
	{
		Alpha = 0.f;
		bInterpolate = false;
	}
	else if (Alpha > 1.f - UE_KINDA_SMALL_NUMBER)
	{
		bInterpolate = false;
		KeyIndex1 = KeyIndex2;
	}

	const float TimePerFrame = SequenceLength / (float)FMath::Max(NumFrames - 1, 1);

	if (bInterpolate)
	{
		BuildPoseFromRawDataInternal<true>(InAnimationData, TrackToSkeletonMapTable, InOutPose, KeyIndex1, KeyIndex2, Alpha, TimePerFrame, AdditiveBoneTransformCurves);
	}
	else
	{
		BuildPoseFromRawDataInternal<false>(InAnimationData, TrackToSkeletonMapTable, InOutPose, KeyIndex1, KeyIndex2, Alpha, TimePerFrame, AdditiveBoneTransformCurves);
	}

	const FBoneContainer& RequiredBones = InOutPose.GetBoneContainer();
	const bool bDisableRetargeting = RequiredBones.GetDisableRetargeting();

	if (!bDisableRetargeting)
	{
		const TArray<UE::Anim::Retargeting::FRetargetTracking>& RetargetTracking = UE::Anim::FBuildRawPoseScratchArea::Get().RetargetTracking;

		const USkeleton* Skeleton = RequiredBones.GetSkeletonAsset();
		for (const UE::Anim::Retargeting::FRetargetTracking& RT : RetargetTracking)
		{
			FAnimationRuntime::RetargetBoneTransform(Skeleton, SourceName, RetargetTransforms, InOutPose[RT.PoseBoneIndex], RT.SkeletonBoneIndex, RT.PoseBoneIndex, RequiredBones, false);
		}
	}
}


