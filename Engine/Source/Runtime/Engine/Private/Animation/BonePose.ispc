// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Transform.isph"
#include "Math/Quat.isph"
#include "Math/WideVector.isph"

export void NormalizeRotations(uniform FTransform Bones[],
								const uniform int NumBones)
{
	uniform int NumBonesBase = NumBones & ~(programCount-1);
	uniform int BoneOffset = programCount / 4;

	for(uniform int BoneIdx = 0; BoneIdx < NumBonesBase; BoneIdx+=programCount)
	{
		uniform WideFVector4 Rotation[4];

		LoadStridedWideFVector4((uniform FVector4 *uniform)&Rotation[0], (uniform FVector4 *uniform)&Bones[BoneIdx].Rotation, 3);
		LoadStridedWideFVector4((uniform FVector4 *uniform)&Rotation[1], (uniform FVector4 *uniform)&Bones[BoneIdx+BoneOffset].Rotation, 3);
		LoadStridedWideFVector4((uniform FVector4 *uniform)&Rotation[2], (uniform FVector4 *uniform)&Bones[BoneIdx+2*BoneOffset].Rotation, 3);
		LoadStridedWideFVector4((uniform FVector4 *uniform)&Rotation[3], (uniform FVector4 *uniform)&Bones[BoneIdx+3*BoneOffset].Rotation, 3);

		uniform WideFVector4 NormalizedRotation[4];
		NormalizedRotation[0] = VectorNormalizeQuaternion(Rotation[0]);
		NormalizedRotation[1] = VectorNormalizeQuaternion(Rotation[1]);
		NormalizedRotation[2] = VectorNormalizeQuaternion(Rotation[2]);
		NormalizedRotation[3] = VectorNormalizeQuaternion(Rotation[3]);

		StoreStridedWideFVector4((uniform FVector4 *uniform)&Bones[BoneIdx].Rotation, (uniform FVector4 *uniform)&NormalizedRotation[0], 3);
		StoreStridedWideFVector4((uniform FVector4 *uniform)&Bones[BoneIdx+BoneOffset].Rotation, (uniform FVector4 *uniform)&NormalizedRotation[1], 3);
		StoreStridedWideFVector4((uniform FVector4 *uniform)&Bones[BoneIdx+2*BoneOffset].Rotation, (uniform FVector4 *uniform)&NormalizedRotation[2], 3);
		StoreStridedWideFVector4((uniform FVector4 *uniform)&Bones[BoneIdx+3*BoneOffset].Rotation, (uniform FVector4 *uniform)&NormalizedRotation[3], 3);
	}

	// If NumBones isn't divisible by the gang size, do the leftover iterations here
	for(uniform int BoneIdx = NumBonesBase; BoneIdx < NumBones; BoneIdx++)
	{
		Bones[BoneIdx].Rotation = VectorNormalizeQuaternion(Bones[BoneIdx].Rotation);
	}
}

export void ResetToAdditiveIdentity(uniform FTransform Bones[],
									const uniform int NumBones)
{
	for(uniform int BoneIdx = 0; BoneIdx < NumBones; BoneIdx++)
	{
		Bones[BoneIdx].Rotation = Vector0001;
		Bones[BoneIdx].Translation = VectorZero;
		Bones[BoneIdx].Scale3D = VectorZero;
	}
}