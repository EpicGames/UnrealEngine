// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/BoneReference.h"
#include "BoneContainer.h"
#include "Animation/Skeleton.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoneReference)

bool FBoneReference::Initialize(const FBoneContainer& RequiredBones)
{
#if WITH_EDITOR
	BoneName = *BoneName.ToString().TrimStartAndEnd();
#endif

	if (RequiredBones.IsValid())
	{
		BoneIndex = RequiredBones.GetPoseBoneIndexForBoneName(BoneName);

		bUseSkeletonIndex = false;

		// If bone name is not found, look into the leader skeleton to see if it's found there.
		// SkeletalMeshes can exclude bones from the leader skeleton, and that's OK.
		// If it's not found in the leader skeleton, the bone does not exist at all! so we should log it.
		if (BoneIndex == INDEX_NONE && BoneName != NAME_None)
		{
			if (USkeleton* SkeletonAsset = RequiredBones.GetSkeletonAsset())
			{
				if (SkeletonAsset->GetReferenceSkeleton().FindBoneIndex(BoneName) == INDEX_NONE)
				{
					UE_LOGF(LogAnimation, Log, "FBoneReference::Initialize BoneIndex for Bone '%ls' does not exist in Skeleton '%ls'",
						*BoneName.ToString(), *GetNameSafe(SkeletonAsset));
				}
			}
		}

		CachedCompactPoseIndex = RequiredBones.MakeCompactPoseIndex(GetMeshPoseIndex(RequiredBones));
	}
	else
	{
		BoneIndex = INDEX_NONE;
		CachedCompactPoseIndex = FCompactPoseBoneIndex(INDEX_NONE);
	}

	return BoneIndex != INDEX_NONE;
}

bool FBoneReference::Initialize(const USkeleton* Skeleton)
{
	if (Skeleton && (BoneName != NAME_None))
	{
#if WITH_EDITOR
		BoneName = *BoneName.ToString().TrimStartAndEnd();
#endif
		BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
		bUseSkeletonIndex = true;
	}
	else
	{
		BoneIndex = INDEX_NONE;
	}

	CachedCompactPoseIndex = FCompactPoseBoneIndex(INDEX_NONE);

	return (BoneIndex != INDEX_NONE);
}

bool FBoneReference::IsValidToEvaluate(const FBoneContainer& RequiredBones) const
{
	return (BoneIndex != INDEX_NONE && RequiredBones.IsValid() && RequiredBones.Contains(BoneIndex));
}

FSkeletonPoseBoneIndex FBoneReference::GetSkeletonPoseIndex(const FBoneContainer& RequiredBones) const
{ 
	// accessing array with invalid index would cause crash, so we have to check here
	if (BoneIndex != INDEX_NONE)
	{
		if (bUseSkeletonIndex)
		{
			return FSkeletonPoseBoneIndex(BoneIndex);
		}
		else
		{
			return RequiredBones.GetSkeletonPoseIndexFromMeshPoseIndex(FMeshPoseBoneIndex(BoneIndex));
		}
	}

	return FSkeletonPoseBoneIndex(INDEX_NONE);
}
	
FMeshPoseBoneIndex FBoneReference::GetMeshPoseIndex(const FBoneContainer& RequiredBones) const
{ 
	// accessing array with invalid index would cause crash, so we have to check here
	if (BoneIndex != INDEX_NONE)
	{
		if (bUseSkeletonIndex)
		{
			return RequiredBones.GetMeshPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(BoneIndex));
		}
		else
		{
			return FMeshPoseBoneIndex(BoneIndex);
		}
	}

	return FMeshPoseBoneIndex(INDEX_NONE);
}

FCompactPoseBoneIndex FBoneReference::GetCompactPoseIndex(const FBoneContainer& RequiredBones) const 
{ 
	if (bUseSkeletonIndex)
	{
		//If we were initialized with a skeleton we wont have a cached index.
		if (BoneIndex != INDEX_NONE)
		{
			// accessing array with invalid index would cause crash, so we have to check here
			return RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(BoneIndex));
		}
		return FCompactPoseBoneIndex(INDEX_NONE);
	}
		
	return CachedCompactPoseIndex;
}

// need this because of BoneReference being used in CurveMetaData and that is in SmartName
FArchive& operator<<(FArchive& Ar, FBoneReference& B)
{
	Ar << B.BoneName;
	return Ar;
}

bool FBoneReference::Serialize(FArchive& Ar)
{
	Ar << *this;
	return true;
}