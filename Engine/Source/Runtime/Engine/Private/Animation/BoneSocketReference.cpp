// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/BoneSocketReference.h"
#include "Animation/AnimInstanceProxy.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMeshSocket.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoneSocketReference)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Socket Reference 
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FSocketReference::InitializeSocketInfo(const FAnimInstanceProxy* InAnimInstanceProxy)
{
	CachedSocketMeshBoneIndex = FMeshPoseBoneIndex(INDEX_NONE);
	CachedSocketCompactBoneIndex = FCompactPoseBoneIndex(INDEX_NONE);

	if (SocketName != NAME_None)
	{
		const USkeletalMeshComponent* OwnerMeshComponent = InAnimInstanceProxy->GetSkelMeshComponent();
		if (OwnerMeshComponent && OwnerMeshComponent->DoesSocketExist(SocketName))
		{
			USkeletalMeshSocket const* const Socket = OwnerMeshComponent->GetSocketByName(SocketName);
			if (Socket)
			{
				CachedSocketLocalTransform = Socket->GetSocketLocalTransform();
				// cache mesh bone index, so that we know this is valid information to follow
				CachedSocketMeshBoneIndex = FMeshPoseBoneIndex(OwnerMeshComponent->GetBoneIndex(Socket->BoneName));

				ensureMsgf(CachedSocketMeshBoneIndex.IsValid(), TEXT("%s : socket has invalid bone."), *SocketName.ToString());
			}
		}
		else
		{
			// @todo : move to graph node warning
			UE_LOGF(LogAnimation, Warning, "%ls: socket doesn't exist", *SocketName.ToString());
		}
	}
}

void FSocketReference::InitialzeCompactBoneIndex(const FBoneContainer& RequiredBones)
{
	if (CachedSocketMeshBoneIndex.IsValid())
	{
		const FSkeletonPoseBoneIndex SocketBoneSkeletonIndex = RequiredBones.GetSkeletonPoseIndexFromMeshPoseIndex(CachedSocketMeshBoneIndex);
		CachedSocketCompactBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(SocketBoneSkeletonIndex);
	}
}


