// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/CachedAnimData.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimStateMachineTypes.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CachedAnimData)

bool FCachedAnimStateData::IsValid(UAnimInstance& InAnimInstance) const
{
	if (!bInitialized)
	{
		bInitialized = true;
		if ((StateMachineName != NAME_None) && (StateName != NAME_None))
		{
			const FBakedAnimationStateMachine* MachineDescription = nullptr;
			InAnimInstance.GetStateMachineIndexAndDescription(StateMachineName, MachineIndex, &MachineDescription);

			if (MachineDescription)
			{
				check(MachineIndex != INDEX_NONE);
				StateIndex = MachineDescription->FindStateIndex(StateName);
				if (StateIndex == INDEX_NONE)
				{
					UE_LOGF(LogAnimation, Warning, "FCachedAnimStateData::GetWeight StateName %ls not found in StateMachineName %ls in AnimBP: %ls. Renamed or deleted?", *StateName.ToString(), *StateMachineName.ToString(), *GetNameSafe(&InAnimInstance));
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "FCachedAnimStateData::GetWeight StateMachineName %ls not found! (With State %ls in AnimBP: %ls) Renamed or deleted?", *StateMachineName.ToString(), *StateName.ToString(), *GetNameSafe(&InAnimInstance));
			}
		}
	}

	return (StateIndex != INDEX_NONE);
}

float FCachedAnimStateData::IsMachineRelevant(UAnimInstance& InAnimInstance) const
{
	return IsValid(InAnimInstance) ? FAnimWeight::IsRelevant(InAnimInstance.GetInstanceMachineWeight(MachineIndex)) : false;
}

float FCachedAnimStateData::GetWeight(UAnimInstance& InAnimInstance) const
{
	return IsValid(InAnimInstance) ? InAnimInstance.GetInstanceStateWeight(MachineIndex, StateIndex) : 0.f;
}

float FCachedAnimStateData::GetGlobalWeight(UAnimInstance& InAnimInstance) const
{
	return IsValid(InAnimInstance) ? (InAnimInstance.GetInstanceMachineWeight(MachineIndex) * InAnimInstance.GetInstanceStateWeight(MachineIndex, StateIndex)) : 0.f;
}

bool FCachedAnimStateData::IsFullWeight(UAnimInstance& InAnimInstance) const
{
	return FAnimWeight::IsFullWeight(GetWeight(InAnimInstance));
}

bool FCachedAnimStateData::IsRelevant(UAnimInstance& InAnimInstance) const
{
	return FAnimWeight::IsRelevant(GetWeight(InAnimInstance));
}

bool FCachedAnimStateData::IsActiveState(class UAnimInstance& InAnimInstance) const
{
	return IsValid(InAnimInstance) ? (InAnimInstance.GetCurrentStateName(MachineIndex) == StateName) : false;
}

bool FCachedAnimStateData::WasAnimNotifyStateActive(UAnimInstance& InAnimInstance, TSubclassOf<UAnimNotifyState> AnimNotifyStateType) const
{
	return IsValid(InAnimInstance) ? (InAnimInstance.WasAnimNotifyStateActiveInSourceState(MachineIndex, StateIndex, AnimNotifyStateType)) : false;
}

bool FCachedAnimStateArray::IsValid(UAnimInstance& InAnimInstance) const
{
	// Make sure the setup validates our assumptions.
	if (!bCheckedValidity)
	{
		bCheckedValidity = true;
		bCachedIsValid = true;

		if (States.Num() > 1)
		{
			FName StateMachineName = NAME_None;
			TArray<TPair<FName,FName>> UniqueStateNames;

			for (const FCachedAnimStateData& State : States)
			{
				if (StateMachineName == NAME_None)
				{
					StateMachineName = State.StateMachineName;
				}
				else if ((State.StateMachineName != NAME_None) && (State.StateMachineName != StateMachineName))
				{
					// Array has states from different state machines
					bHasMultipleStateMachineEntries = true;
				}
				
				TPair<FName, FName> StateName = TPair<FName, FName>(State.StateMachineName, State.StateName);
				if (!UniqueStateNames.Contains(StateName))
				{
					UniqueStateNames.Add(StateName);
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "FCachedAnimStateArray::IsValid StateName %ls from state machine %ls included multiple times in AnimBP: %ls.", *State.StateName.ToString(), *State.StateMachineName.ToString(), *GetNameSafe(&InAnimInstance));
					bCachedIsValid = false;
				}
			}
		}
	}

	return bCachedIsValid;
}

float FCachedAnimStateArray::GetTotalWeight(UAnimInstance& InAnimInstance) const
{
	if (IsValid(InAnimInstance))
	{
		float TotalWeight = 0.f;
		for (const FCachedAnimStateData& State : States)
		{
			TotalWeight += State.GetWeight(InAnimInstance);
		}
		// Don't clamp to 1 if total is composed from multiple state machines since it can be larger than that
		if (bHasMultipleStateMachineEntries)
		{
			return TotalWeight;
		}
		else
		{
			return FMath::Min(TotalWeight, 1.f);
		}
	}
	return 0.f;
}

bool FCachedAnimStateArray::IsFullWeight(UAnimInstance& InAnimInstance) const
{
	if (bHasMultipleStateMachineEntries)
	{
		// Array has multiple state machines, return true if one machine has a weight of 1.
		if (IsValid(InAnimInstance))
		{
			TMap<FName, float> TotalWeights;
			for (const FCachedAnimStateData& State : States)
			{
				TotalWeights.FindOrAdd(State.StateMachineName) += State.GetWeight(InAnimInstance);
			}
			for (const auto& Iter : TotalWeights)
			{
				if (FAnimWeight::IsFullWeight(Iter.Value))
				{
					return true;
				}
			}
		}
		return false;
	}
	else
	{
		return FAnimWeight::IsFullWeight(GetTotalWeight(InAnimInstance));
	}
}

bool FCachedAnimStateArray::IsRelevant(UAnimInstance& InAnimInstance) const
{
	if (IsValid(InAnimInstance))
	{
		for (const FCachedAnimStateData& State : States)
		{
			if (State.IsRelevant(InAnimInstance))
			{
				return true;
			}
		}
	}

	return false;
}

void FCachedAnimAssetPlayerData::CacheIndices(UAnimInstance& InAnimInstance) const
{
	if(!bInitialized)
	{
		bInitialized = true;
		if((StateMachineName != NAME_None) && (StateName != NAME_None))
		{
			Index = InAnimInstance.GetInstanceAssetPlayerIndex(StateMachineName, StateName);
			if(Index == INDEX_NONE)
			{
				UE_LOGF(LogAnimation, Warning, "FCachedAnimAssetPlayerData::GetAssetPlayerTime StateName %ls not found in StateMachineName %ls in AnimBP: %ls. Renamed or deleted?", *StateName.ToString(), *StateMachineName.ToString(), *GetNameSafe(&InAnimInstance));
			}
		}
	}
}

float FCachedAnimAssetPlayerData::GetAssetPlayerTime(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if (Index != INDEX_NONE)
	{
		return InAnimInstance.GetInstanceAssetPlayerTime(Index);
	}

	return 0.f;
}

float FCachedAnimAssetPlayerData::GetAssetPlayerTimeRatio(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if(Index != INDEX_NONE)
	{
		return InAnimInstance.GetInstanceAssetPlayerTimeFraction(Index);
	}

	return 0.0f;
}

void FCachedAnimRelevancyData::CacheIndices(UAnimInstance& InAnimInstance) const
{
	if (!bInitialized)
	{
		bInitialized = true;
		if ((StateMachineName != NAME_None) && (StateName != NAME_None))
		{
			if (MachineIndex == INDEX_NONE)
			{
				MachineIndex = InAnimInstance.GetStateMachineIndex(StateMachineName);
				if (MachineIndex == INDEX_NONE)
				{
					UE_LOGF(LogAnimation, Warning, "FCachedAnimRelevancyData::CacheIndices StateMachineName %ls not found in AnimBP: %ls. Renamed or deleted?", *StateMachineName.ToString(), *GetNameSafe(&InAnimInstance));
				}
			}
			if (StateIndex == INDEX_NONE)
			{
				const FBakedAnimationStateMachine* MachinePtr = InAnimInstance.GetStateMachineInstanceDesc(StateMachineName);
				if (MachinePtr)
				{
					StateIndex = MachinePtr->FindStateIndex(StateName);
					if (StateIndex == INDEX_NONE)
					{
						UE_LOGF(LogAnimation, Warning, "FCachedAnimRelevancyData::CacheIndices StateName %ls not found in StateMachineName %ls in AnimBP: %ls. Renamed or deleted?", *StateName.ToString(), *StateMachineName.ToString(), *GetNameSafe(&InAnimInstance));
					}
				}
			}
		}
	}
}

float FCachedAnimRelevancyData::GetRelevantAnimTime(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if (MachineIndex != INDEX_NONE && StateIndex != INDEX_NONE)
	{
		return InAnimInstance.GetRelevantAnimTime(MachineIndex, StateIndex);
	}

	return 0.f;
}

float FCachedAnimRelevancyData::GetRelevantAnimTimeFraction(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if (MachineIndex != INDEX_NONE && StateIndex != INDEX_NONE)
	{
		return InAnimInstance.GetRelevantAnimTimeFraction(MachineIndex, StateIndex);
	}

	return 0.f;
}

float FCachedAnimRelevancyData::GetRelevantAnimTimeRemaining(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if (MachineIndex != INDEX_NONE && StateIndex != INDEX_NONE)
	{
		return InAnimInstance.GetRelevantAnimTimeRemaining(MachineIndex, StateIndex);
	}

	return 0.f;
}

float FCachedAnimRelevancyData::GetRelevantAnimTimeRemainingFraction(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if (MachineIndex != INDEX_NONE && StateIndex != INDEX_NONE)
	{
		return InAnimInstance.GetRelevantAnimTimeRemainingFraction(MachineIndex, StateIndex);
	}

	return 0.f;
}

void FCachedAnimTransitionData::CacheIndices(UAnimInstance& InAnimInstance) const
{
	if (!bInitialized)
	{
		bInitialized = true;
		if ((StateMachineName != NAME_None) && (FromStateName != NAME_None) && (ToStateName != NAME_None))
		{
			if (MachineIndex == INDEX_NONE)
			{
				MachineIndex = InAnimInstance.GetStateMachineIndex(StateMachineName);
				if (MachineIndex == INDEX_NONE)
				{
					UE_LOGF(LogAnimation, Warning, "FCachedAnimTransitionData::CacheIndices StateMachineName %ls in AnimBP: %ls not found. Renamed or deleted?", *StateMachineName.ToString(), *GetNameSafe(&InAnimInstance));
				}
			}
			if (TransitionIndex == INDEX_NONE)
			{
				const FBakedAnimationStateMachine* MachinePtr = InAnimInstance.GetStateMachineInstanceDesc(StateMachineName);
				if (MachinePtr)
				{
					TransitionIndex = MachinePtr->FindTransitionIndex(FromStateName, ToStateName);
					if (TransitionIndex == INDEX_NONE)
					{
						UE_LOGF(LogAnimation, Warning, "FCachedAnimTransitionData::CacheIndices Transition from %ls to %ls not found in StateMachineName %ls in AnimBP: %ls. Renamed or deleted?", *FromStateName.ToString(), *ToStateName.ToString(), *StateMachineName.ToString(), *GetNameSafe(&InAnimInstance));
					}
				}
			}
		}
	}
}

float FCachedAnimTransitionData::GetCrossfadeDuration(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if (MachineIndex != INDEX_NONE && TransitionIndex != INDEX_NONE)
	{
		return InAnimInstance.GetInstanceTransitionCrossfadeDuration(MachineIndex, TransitionIndex);
	}

	return 0.f;
}

float FCachedAnimTransitionData::GetTimeElapsed(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if (MachineIndex != INDEX_NONE && TransitionIndex != INDEX_NONE)
	{
		return InAnimInstance.GetInstanceTransitionTimeElapsed(MachineIndex, TransitionIndex);
	}

	return 0.f;
}

float FCachedAnimTransitionData::GetTimeElapsedFraction(UAnimInstance& InAnimInstance) const
{
	CacheIndices(InAnimInstance);

	if (MachineIndex != INDEX_NONE && TransitionIndex != INDEX_NONE)
	{
		return InAnimInstance.GetInstanceTransitionTimeElapsedFraction(MachineIndex, TransitionIndex);
	}

	return 0.f;
}


