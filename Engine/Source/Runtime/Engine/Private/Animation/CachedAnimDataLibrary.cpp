// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/CachedAnimDataLibrary.h"
#include "Animation/CachedAnimData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CachedAnimDataLibrary)

UCachedAnimDataLibrary::UCachedAnimDataLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UCachedAnimDataLibrary::StateMachine_IsStateRelevant(UAnimInstance* InAnimInstance, const FCachedAnimStateData& CachedAnimStateData)
{
	if (InAnimInstance)
	{
		return CachedAnimStateData.IsRelevant(*InAnimInstance);
	}

	return false;
}

float UCachedAnimDataLibrary::StateMachine_GetLocalWeight(UAnimInstance* InAnimInstance, const FCachedAnimStateData& CachedAnimStateData)
{
	if (InAnimInstance)
	{
		return CachedAnimStateData.GetWeight(*InAnimInstance);
	}

	return 0.0f;
}

float UCachedAnimDataLibrary::StateMachine_GetGlobalWeight(UAnimInstance* InAnimInstance, const FCachedAnimStateData& CachedAnimStateData)
{
	if (InAnimInstance)
	{
		return CachedAnimStateData.GetGlobalWeight(*InAnimInstance);
	}

	return 0.0f;
}

float UCachedAnimDataLibrary::StateMachine_GetTotalWeight(UAnimInstance* InAnimInstance, const FCachedAnimStateArray& CachedAnimStateArray)
{
	if (InAnimInstance)
	{
		return CachedAnimStateArray.GetTotalWeight(*InAnimInstance);
	}

	return 0.0f;
}

bool UCachedAnimDataLibrary::StateMachine_IsFullWeight(UAnimInstance* InAnimInstance, const FCachedAnimStateArray& CachedAnimStateArray)
{
	if (InAnimInstance)
	{
		return CachedAnimStateArray.IsFullWeight(*InAnimInstance);
	}

	return false;
}

bool UCachedAnimDataLibrary::StateMachine_IsRelevant(UAnimInstance* InAnimInstance, const FCachedAnimStateArray& CachedAnimStateArray)
{
	if (InAnimInstance)
	{
		return CachedAnimStateArray.IsRelevant(*InAnimInstance);
	}

	return false;
}

float UCachedAnimDataLibrary::StateMachine_GetAssetPlayerTime(UAnimInstance* InAnimInstance, const FCachedAnimAssetPlayerData& CachedAnimAssetPlayerData)
{
	if (InAnimInstance)
	{
		return CachedAnimAssetPlayerData.GetAssetPlayerTime(*InAnimInstance);
	}

	return 0.0f;
}

float UCachedAnimDataLibrary::StateMachine_GetAssetPlayerTimeRatio(UAnimInstance* InAnimInstance, const FCachedAnimAssetPlayerData& CachedAnimAssetPlayerData)
{
	if (InAnimInstance)
	{
		return CachedAnimAssetPlayerData.GetAssetPlayerTimeRatio(*InAnimInstance);
	}

	return 0.0f;
}

float UCachedAnimDataLibrary::StateMachine_GetRelevantAnimTime(UAnimInstance* InAnimInstance, const FCachedAnimRelevancyData& CachedAnimRelevancyData)
{
	if (InAnimInstance)
	{
		return CachedAnimRelevancyData.GetRelevantAnimTime(*InAnimInstance);
	}

	return 0.0f;
}

float UCachedAnimDataLibrary::StateMachine_GetRelevantAnimTimeRemaining(UAnimInstance* InAnimInstance, const FCachedAnimRelevancyData& CachedAnimRelevancyData)
{
	if (InAnimInstance)
	{
		return CachedAnimRelevancyData.GetRelevantAnimTimeRemaining(*InAnimInstance);
	}

	return 0.0f;
}

float UCachedAnimDataLibrary::StateMachine_GetRelevantAnimTimeRemainingFraction(UAnimInstance* InAnimInstance, const FCachedAnimRelevancyData& CachedAnimRelevancyData)
{
	if (InAnimInstance)
	{
		return CachedAnimRelevancyData.GetRelevantAnimTimeRemainingFraction(*InAnimInstance);
	}

	return 0.0f;
}

float UCachedAnimDataLibrary::StateMachine_GetCrossfadeDuration(UAnimInstance* InAnimInstance, const FCachedAnimTransitionData& CachedAnimTransitionData)
{
	if (InAnimInstance)
	{
		return CachedAnimTransitionData.GetCrossfadeDuration(*InAnimInstance);
	}

	return 0.0f;
}
