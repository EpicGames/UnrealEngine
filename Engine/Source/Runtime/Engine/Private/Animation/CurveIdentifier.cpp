// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimData/CurveIdentifier.h"
#include "Animation/Skeleton.h"
#include "EngineLogs.h"
#include "Animation/Skeleton.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveIdentifier)

void FAnimationCurveIdentifier::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if(Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimationRemoveSmartNames)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CurveName = InternalName_DEPRECATED.DisplayName;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif
}

#if WITH_EDITOR
void UAnimationCurveIdentifierExtensions::SetCurveIdentifier(FAnimationCurveIdentifier& InOutIdentifier, FName Name, ERawCurveTrackTypes CurveType)
{
	InOutIdentifier.CurveName = Name;
	InOutIdentifier.CurveType = CurveType;
}

FAnimationCurveIdentifier UAnimationCurveIdentifierExtensions::GetCurveIdentifier(USkeleton* InSkeleton, FName Name, ERawCurveTrackTypes CurveType)
{
	FAnimationCurveIdentifier Identifier;
	Identifier.CurveName = Name;
	Identifier.CurveType = CurveType;

	return Identifier;
}

FAnimationCurveIdentifier UAnimationCurveIdentifierExtensions::FindCurveIdentifier(const USkeleton* InSkeleton, FName Name, ERawCurveTrackTypes CurveType)
{
	FAnimationCurveIdentifier Identifier;
	Identifier.CurveName = Name;
	Identifier.CurveType = CurveType;

	return Identifier;
}

TArray<FAnimationCurveIdentifier> UAnimationCurveIdentifierExtensions::GetCurveIdentifiers(USkeleton* InSkeleton, ERawCurveTrackTypes CurveType)
{
	TArray<FAnimationCurveIdentifier> Identifiers;
	return Identifiers;
}

bool UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(FAnimationCurveIdentifier& InOutIdentifier, ETransformCurveChannel Channel, EVectorCurveChannel Axis)
{
	if (InOutIdentifier.IsValid())
	{
		if (InOutIdentifier.CurveType == ERawCurveTrackTypes::RCT_Transform)
		{
			InOutIdentifier.Channel = Channel;
			InOutIdentifier.Axis = Axis;

			return true;
		}
	}

	return false;
}
#endif // WITH_EDITOR


