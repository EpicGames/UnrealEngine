// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/ExposedValueHandler.h"
#include "Animation/AnimSubsystem_PropertyAccess.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExposedValueHandler)

// Deprecation support
PRAGMA_DISABLE_DEPRECATION_WARNINGS
TArray<FExposedValueCopyRecord> FExposedValueHandler::CopyRecords;
TObjectPtr<UFunction> FExposedValueHandler::Function;
const FPropertyAccessLibrary* FExposedValueHandler::PropertyAccessLibrary;
FName FExposedValueHandler::BoundFunction;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FExposedValueHandler::ClassInitialization(TArray<FExposedValueHandler>& InHandlers, UClass* InClass)
{
	if(const FAnimSubsystem_PropertyAccess* Subsystem = IAnimClassInterface::GetFromClass(InClass)->FindSubsystem<FAnimSubsystem_PropertyAccess>())
	{
		const FPropertyAccessLibrary& Library = Subsystem->GetLibrary();

		for(FExposedValueHandler& Handler : InHandlers)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Handler.Initialize(InClass, Library);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

void FExposedValueHandler::Initialize(UClass* InClass, const FPropertyAccessLibrary& InPropertyAccessLibrary)
{
	if(Handler)
	{
		Handler->Initialize(InClass);
	}
}

// Don't inline this function to keep the stack usage down
FORCENOINLINE void FExposedValueHandler::Execute(const FAnimationBaseContext& Context) const
{
	if (Handler)
	{
		Handler->Execute(Context);
	}
}

void FAnimNodeExposedValueHandler_Base::Initialize(const UClass* InClass)
{
	if (BoundFunction != NAME_None)
	{
		// This cached function is nullptr when the CDO is initially serialized, or (in editor) when the class has been
		// recompiled and any instances have been re-instanced. When new instances are spawned, this function is
		// duplicated (it is a FProperty) onto those instances so we dont pay the cost of the FindFunction() call
#if !WITH_EDITOR
		if (Function == nullptr)
#endif
		{
			Function = InClass->FindFunctionByName(BoundFunction);
			check(Function);
		}
	}
	else
	{
		Function = nullptr;
	}
}

void FAnimNodeExposedValueHandler_Base::Execute(const FAnimationBaseContext& InContext) const
{
	if (Function != nullptr)
	{
		InContext.AnimInstanceProxy->GetAnimInstanceObject()->ProcessEvent(Function, nullptr);
	}
}

void FAnimNodeExposedValueHandler_PropertyAccess::Initialize(const UClass* InClass)
{
	Super::Initialize(InClass);

	if (const FAnimSubsystem_PropertyAccess* Subsystem = IAnimClassInterface::GetFromClass(InClass)->FindSubsystem<FAnimSubsystem_PropertyAccess>())
	{
		PropertyAccessLibrary = &Subsystem->GetLibrary();
	}
}

void FAnimNodeExposedValueHandler_PropertyAccess::Execute(const FAnimationBaseContext& InContext) const
{
	Super::Execute(InContext);

	if (CopyRecords.Num() > 0)
	{
		if (PropertyAccessLibrary != nullptr)
		{
			UObject* AnimInstanceObject = InContext.AnimInstanceProxy->GetAnimInstanceObject();
			PropertyAccess::FCopyBatchId CopyBatch((int32)EAnimPropertyAccessCallSite::WorkerThread_Unbatched);
			for (const FExposedValueCopyRecord& CopyRecord : CopyRecords)
			{
				if (!CopyRecord.bOnlyUpdateWhenActive || InContext.IsActive())
				{
					PropertyAccess::ProcessCopy(AnimInstanceObject, *PropertyAccessLibrary, CopyBatch, CopyRecord.CopyIndex, [&CopyRecord](const FProperty* InProperty, void* InAddress)
					{
						if (CopyRecord.PostCopyOperation == EPostCopyOperation::LogicalNegateBool)
						{
							bool bValue = static_cast<const FBoolProperty*>(InProperty)->GetPropertyValue(InAddress);
							static_cast<const FBoolProperty*>(InProperty)->SetPropertyValue(InAddress, !bValue);
						}
					});
				}
			}
		}
	}
}

