// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/InputScaleBias.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputScaleBias)

#define LOCTEXT_NAMESPACE "FInputScaleBias"

/////////////////////////////////////////////////////
// FInputScaleBias

float FInputScaleBias::ApplyTo(float Value) const
{
	return FMath::Clamp<float>( Value * Scale + Bias, 0.0f, 1.0f );
}

#if WITH_EDITOR
FText FInputScaleBias::GetFriendlyName(FText InFriendlyName) const
{
	FText OutFriendlyName = InFriendlyName;

	if (Scale != 1.f)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("Scale"), FText::AsNumber(Scale));

		if (Scale == -1.f)
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Scale", "- {PinFriendlyName}"), Args);
		}
		else
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_ScaleMul", "{Scale} * {PinFriendlyName}"), Args);
		}
	}

	if (Bias != 0.f)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("Bias"), FText::AsNumber(Bias));

		// '-' Sign already included in Scale above.
		if (Scale < 0.f)
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Bias", "{Bias} {PinFriendlyName}"), Args);
		}
		else
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_BiasPlus", "{Bias} + {PinFriendlyName}"), Args);
		}
	}

	return OutFriendlyName;
}
#endif

/////////////////////////////////////////////////////
// FInputClamp

#if WITH_EDITOR
FText FInputClampConstants::GetFriendlyName(FText InFriendlyName) const
{
	FText OutFriendlyName = InFriendlyName;

	// Clamp
	if (bClampResult)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("ClampMin"), ClampMin);
		Args.Add(TEXT("ClampMax"), ClampMax);
		OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Clamp", "Clamp({PinFriendlyName}, {ClampMin}, {ClampMax})"), Args);
	}

	// Interp
	if (bInterpResult)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("InterpSpeedIncreasing"), InterpSpeedIncreasing);
		Args.Add(TEXT("InterpSpeedDecreasing"), InterpSpeedDecreasing);
		OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Interp", "FInterp({PinFriendlyName}, ({InterpSpeedIncreasing}:{InterpSpeedDecreasing}))"), Args);
	}

	return OutFriendlyName;
}
#endif

float FInputClampState::ApplyTo(const FInputClampConstants& InConstants, float InValue, float InDeltaTime)
{
	float Result = InValue;

	if (InConstants.bClampResult)
	{
		Result = FMath::Clamp<float>(Result, InConstants.ClampMin, InConstants.ClampMax);
	}

	if (InConstants.bInterpResult)
	{
		if (bInitialized)
		{
			const float InterpSpeed = (Result >= InterpolatedResult) ? InConstants.InterpSpeedIncreasing : InConstants.InterpSpeedDecreasing;
			Result = FMath::FInterpTo(InterpolatedResult, Result, InDeltaTime, InterpSpeed);
		}

		InterpolatedResult = Result;
	}

	bInitialized = true;
	return Result;
}

/////////////////////////////////////////////////////
// FInputScaleBiasClamp

float FInputScaleBiasClamp::ApplyTo(float Value, float InDeltaTime) const
{
	float Result = Value;

	if (bMapRange)
	{
		Result = FMath::GetMappedRangeValueUnclamped(InRange.ToVector2f(), OutRange.ToVector2f(), Result);
	}

	Result = Result * Scale + Bias;

	if (bClampResult)
	{
		Result = FMath::Clamp<float>(Result, ClampMin, ClampMax);
	}

	if (bInterpResult)
	{
		if (bInitialized)
		{
			const float InterpSpeed = (Result >= InterpolatedResult) ? InterpSpeedIncreasing : InterpSpeedDecreasing;
			Result = FMath::FInterpTo(InterpolatedResult, Result, InDeltaTime, InterpSpeed);
		}

		InterpolatedResult = Result;
	}

	bInitialized = true;
	return Result;
}

#if WITH_EDITOR
FText FInputScaleBiasClamp::GetFriendlyName(FText InFriendlyName) const
{
	FText OutFriendlyName = InFriendlyName;

	// MapRange
	if (bMapRange)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("InRangeMin"), InRange.Min);
		Args.Add(TEXT("InRangeMax"), InRange.Max);
		Args.Add(TEXT("OutRangeMin"), OutRange.Min);
		Args.Add(TEXT("OutRangeMax"), OutRange.Max);
		OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_MapRange", "MapRange({PinFriendlyName}, In({InRangeMin}:{InRangeMax}), Out({OutRangeMin}:{OutRangeMax}))"), Args);
	}

	if (Scale != 1.f)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("Scale"), FText::AsNumber(Scale));

		if (Scale == -1.f)
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Scale", "- {PinFriendlyName}"), Args);
		}
		else 
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_ScaleMul", "{Scale} * {PinFriendlyName}"), Args);
		}
	}

	if (Bias != 0.f)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("Bias"), FText::AsNumber(Bias));

		// '-' Sign already included in Scale above.
		if (Scale < 0.f)
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Bias", "{Bias} {PinFriendlyName}"), Args);
		}
		else
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_BiasPlus", "{Bias} + {PinFriendlyName}"), Args);
		}
	}

	// Clamp
	if (bClampResult)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("ClampMin"), ClampMin);
		Args.Add(TEXT("ClampMax"), ClampMax);
		OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Clamp", "Clamp({PinFriendlyName}, {ClampMin}, {ClampMax})"), Args);
	}

	// Interp
	if (bInterpResult)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("InterpSpeedIncreasing"), InterpSpeedIncreasing);
		Args.Add(TEXT("InterpSpeedDecreasing"), InterpSpeedDecreasing);
		OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Interp", "FInterp({PinFriendlyName}, ({InterpSpeedIncreasing}:{InterpSpeedDecreasing}))"), Args);
	}

	return OutFriendlyName;
}
#endif

float FInputScaleBiasClampState::ApplyTo(const FInputScaleBiasClampConstants& InConstants, float InValue, float InDeltaTime)
{
	float Result = InValue;

	if (InConstants.bMapRange)
	{
		Result = FMath::GetMappedRangeValueUnclamped(InConstants.InRange.ToVector2f(), InConstants.OutRange.ToVector2f(), Result);
	}

	Result = Result * InConstants.Scale + InConstants.Bias;

	if (InConstants.bClampResult)
	{
		Result = FMath::Clamp<float>(Result, InConstants.ClampMin, InConstants.ClampMax);
	}

	if (InConstants.bInterpResult)
	{
		if (bInitialized)
		{
			const float InterpSpeed = (Result >= InterpolatedResult) ? InConstants.InterpSpeedIncreasing : InConstants.InterpSpeedDecreasing;
			Result = FMath::FInterpTo(InterpolatedResult, Result, InDeltaTime, InterpSpeed);
		}

		InterpolatedResult = Result;
	}

	bInitialized = true;
	return Result;
}

float FInputScaleBiasClampState::ApplyTo(const FInputScaleBiasClampConstants& InConstants, float InValue) const
{
	float Result = InValue;

	if (InConstants.bMapRange)
	{
		Result = FMath::GetMappedRangeValueUnclamped(InConstants.InRange.ToVector2f(), InConstants.OutRange.ToVector2f(), Result);
	}

	Result = Result * InConstants.Scale + InConstants.Bias;

	if (InConstants.bClampResult)
	{
		Result = FMath::Clamp<float>(Result, InConstants.ClampMin, InConstants.ClampMax);
	}

	return Result;
}

#if WITH_EDITORONLY_DATA
bool FInputAlphaBoolBlend::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}

	return false;
}

bool FInputAlphaBoolBlend::Serialize(FStructuredArchive::FSlot Slot)
{
	FArchive& Ar = Slot.GetUnderlyingArchive();
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}
	return false;
}
#endif // #if WITH_EDITORONLY_DATA

#if WITH_EDITOR
FText FInputScaleBiasClampConstants::GetFriendlyName(FText InFriendlyName) const
{
	FText OutFriendlyName = InFriendlyName;

	// MapRange
	if (bMapRange)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("InRangeMin"), InRange.Min);
		Args.Add(TEXT("InRangeMax"), InRange.Max);
		Args.Add(TEXT("OutRangeMin"), OutRange.Min);
		Args.Add(TEXT("OutRangeMax"), OutRange.Max);
		OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_MapRange", "MapRange({PinFriendlyName}, In({InRangeMin}:{InRangeMax}), Out({OutRangeMin}:{OutRangeMax}))"), Args);
	}

	if (Scale != 1.f)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("Scale"), FText::AsNumber(Scale));

		if (Scale == -1.f)
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Scale", "- {PinFriendlyName}"), Args);
		}
		else 
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_ScaleMul", "{Scale} * {PinFriendlyName}"), Args);
		}
	}

	if (Bias != 0.f)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("Bias"), FText::AsNumber(Bias));

		// '-' Sign already included in Scale above.
		if (Scale < 0.f)
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Bias", "{Bias} {PinFriendlyName}"), Args);
		}
		else
		{
			OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_BiasPlus", "{Bias} + {PinFriendlyName}"), Args);
		}
	}

	// Clamp
	if (bClampResult)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("ClampMin"), ClampMin);
		Args.Add(TEXT("ClampMax"), ClampMax);
		OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Clamp", "Clamp({PinFriendlyName}, {ClampMin}, {ClampMax})"), Args);
	}

	// Interp
	if (bInterpResult)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinFriendlyName"), OutFriendlyName);
		Args.Add(TEXT("InterpSpeedIncreasing"), InterpSpeedIncreasing);
		Args.Add(TEXT("InterpSpeedDecreasing"), InterpSpeedDecreasing);
		OutFriendlyName = FText::Format(LOCTEXT("FInputScaleBias_Interp", "FInterp({PinFriendlyName}, ({InterpSpeedIncreasing}:{InterpSpeedDecreasing}))"), Args);
	}

	return OutFriendlyName;
}

void FInputScaleBiasClampConstants::CopyFromLegacy(const FInputScaleBiasClamp& InLegacy)
{
	bMapRange = InLegacy.bMapRange;
	bClampResult = InLegacy.bClampResult;
	bInterpResult = InLegacy.bInterpResult;
	InRange = InLegacy.InRange;
	OutRange = InLegacy.OutRange;
	Scale = InLegacy.Scale;
	Bias = InLegacy.Bias;
	ClampMin = InLegacy.ClampMin;
	ClampMax = InLegacy.ClampMax;
	InterpSpeedIncreasing = InLegacy.InterpSpeedIncreasing;
	InterpSpeedDecreasing = InLegacy.InterpSpeedDecreasing;
}
#endif

/////////////////////////////////////////////////////
// FInputAlphaBool

float FInputAlphaBoolBlend::ApplyTo(bool bEnabled, float InDeltaTime)
{
	const float TargetValue = bEnabled ? 1.f : 0.f;

	if (!bInitialized)
	{
		if (CustomCurve != AlphaBlend.GetCustomCurve())
		{
			AlphaBlend.SetCustomCurve(CustomCurve);
		}

		if (BlendOption != AlphaBlend.GetBlendOption())
		{
			AlphaBlend.SetBlendOption(BlendOption);
		}

		AlphaBlend.SetDesiredValue(TargetValue);
		AlphaBlend.SetBlendTime(0.f);
		AlphaBlend.Reset();
		bInitialized = true;
	}
	else
	{
		if (AlphaBlend.GetDesiredValue() != TargetValue)
		{
			AlphaBlend.SetDesiredValue(TargetValue);
			AlphaBlend.SetBlendTime(bEnabled ? BlendInTime : BlendOutTime);
		}
	}

	AlphaBlend.Update(InDeltaTime);
	return AlphaBlend.GetBlendedValue();
}

#undef LOCTEXT_NAMESPACE 
