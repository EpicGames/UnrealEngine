// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && INTEL_ISPC

#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "Misc/AutomationTest.h"
#include "AnimEncoding_VariableKeyLerp.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIspcTestAnimEncodingVariableKeyLerpGetPoseRotations, "Ispc.Animation.VariableKeyLerp.GetPoseRotations", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FIspcTestAnimEncodingVariableKeyLerpGetPoseRotations::RunTest(const FString& Parameters)
{
	const FString CommandName("a.VariableKeyLerp.ISPC");
	auto FormatCommand = [CommandName](bool State) -> FString {
		return FString::Format(TEXT("{0} {1}"), { CommandName, State });
	};

	const IConsoleVariable* CVarISPCEnabled = IConsoleManager::Get().FindConsoleVariable(*CommandName);
	bool InitialState = CVarISPCEnabled->GetBool();
	check(GEngine);

	TArray<FTransform> ISPCTransforms;
	ISPCTransforms.Add(FTransform(FQuat4d(45., 0., 0., 1.)));
	ISPCTransforms.Add(FTransform(FQuat4d(-450., 180., 400., 42.)));
	ISPCTransforms.Add(FTransform(FQuat4d(90., 180., 0., 1.)));
	ISPCTransforms.Add(FTransform(FQuat4d(0., 0., 100., 1.)));
	ISPCTransforms.Add(FTransform(FQuat4d(100., 100., 100., 1.)));
	ISPCTransforms.Add(FTransform(FQuat4d(0., 0., 0., 1.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.)));
	ISPCTransforms.Add(FTransform(FQuat4d(-1., 1., -1., 1.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., -1., 1., 1.)));

	TArray<FTransform> CPPTransforms(ISPCTransforms);
	TArrayView<FTransform> ISPCAtoms(ISPCTransforms);
	TArrayView<FTransform> CPPAtoms(CPPTransforms);

	BoneTrackArray Bones;
	Bones.Add(BoneTrackPair(0, 0));
	Bones.Add(BoneTrackPair(1, 1));
	Bones.Add(BoneTrackPair(2, 2));
	
	TArray<uint8> BulkData;
	BulkData.Init(10, 100);
	FCompressedOffsetDataBase<TArray<int32>> ScaleOffets;
	ScaleOffets.OffsetData.Init(30, 100);

	FUECompressedAnimDataMutable CompressedData;
	CompressedData.CompressedTrackOffsets.Init(30, 100);
	CompressedData.CompressedScaleOffsets = ScaleOffets;
	CompressedData.CompressedByteStream = BulkData;
	CompressedData.KeyEncodingFormat = AKF_VariableKeyLerp;
	CompressedData.RotationCompressionFormat = ACF_Identity;
	CompressedData.TranslationCompressionFormat = ACF_Identity;
	CompressedData.ScaleCompressionFormat = ACF_Identity;
	CompressedData.BuildFinalBuffer(BulkData);

	FName AnimName;
	FFrameRate FrameRate;
	TArray<FTrackToSkeletonMap> SkeletonMap;

	GEngine->Exec(nullptr, *FormatCommand(true));
	{
		AEFVariableKeyLerp<ACF_Identity> KeyLerp;
		FAnimSequenceDecompressionContext DecompressContext(FrameRate,
			1,
			EAnimInterpolationType::Linear,
			AnimName,
			CompressedData,
			ISPCTransforms,
			SkeletonMap,
			nullptr,
			false,
			AAT_None);
		KeyLerp.GetPoseRotations(ISPCAtoms, Bones, DecompressContext);
	}

	GEngine->Exec(nullptr, *FormatCommand(false));
	{
		AEFVariableKeyLerp<ACF_Identity> KeyLerp;
		FAnimSequenceDecompressionContext DecompressContext(FrameRate,
			1,
			EAnimInterpolationType::Linear,
			AnimName,
			CompressedData,
			CPPTransforms,
			SkeletonMap,
			nullptr,
			false,
			AAT_None);
		KeyLerp.GetPoseRotations(CPPAtoms, Bones, DecompressContext);
	}

	GEngine->Exec(nullptr, *FormatCommand(InitialState));

	for (int32 i = 0; i < ISPCAtoms.Num(); ++i)
	{
		const FString Message(FString::Format(TEXT("Quat {0}"), { i }));
		TestTrue(*Message, ISPCAtoms[i].GetRotation().Equals(CPPAtoms[i].GetRotation()));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIspcTestAnimEncodingVariableKeyLerpGetPoseTranslations, "Ispc.Animation.VariableKeyLerp.GetPoseTranslations", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FIspcTestAnimEncodingVariableKeyLerpGetPoseTranslations::RunTest(const FString& Parameters)
{
	const FString CommandName("a.VariableKeyLerp.ISPC");
	auto FormatCommand = [CommandName](bool State) -> FString {
		return FString::Format(TEXT("{0} {1}"), { CommandName, State });
	};

	const IConsoleVariable* CVarISPCEnabled = IConsoleManager::Get().FindConsoleVariable(*CommandName);
	bool InitialState = CVarISPCEnabled->GetBool();
	check(GEngine);

	TArray<FTransform> ISPCTransforms;
	ISPCTransforms.Add(FTransform(FVector3d(45., 0., 0.)));
	ISPCTransforms.Add(FTransform(FVector3d(-450., 180., 400.)));
	ISPCTransforms.Add(FTransform(FVector3d(90., 180., 0.)));
	ISPCTransforms.Add(FTransform(FVector3d(0., 0., 100.)));
	ISPCTransforms.Add(FTransform(FVector3d(100., 100., 100.)));
	ISPCTransforms.Add(FTransform(FVector3d(0., 0., 0.)));
	ISPCTransforms.Add(FTransform(FVector3d(1., 1., 1.)));
	ISPCTransforms.Add(FTransform(FVector3d(-1., 1., -1.)));
	ISPCTransforms.Add(FTransform(FVector3d(1., -1., 1.)));

	TArray<FTransform> CPPTransforms(ISPCTransforms);
	TArrayView<FTransform> ISPCAtoms(ISPCTransforms);
	TArrayView<FTransform> CPPAtoms(CPPTransforms);

	BoneTrackArray Bones;
	Bones.Add(BoneTrackPair(0, 0));
	Bones.Add(BoneTrackPair(1, 1));
	Bones.Add(BoneTrackPair(2, 2));

	TArray<uint8> BulkData;
	BulkData.Init(10, 100);
	FCompressedOffsetDataBase<TArray<int32>> ScaleOffets;
	ScaleOffets.OffsetData.Init(30, 100);

	FUECompressedAnimDataMutable CompressedData;
	CompressedData.CompressedTrackOffsets.Init(30, 100);
	CompressedData.CompressedScaleOffsets = ScaleOffets;
	CompressedData.CompressedByteStream = BulkData;
	CompressedData.KeyEncodingFormat = AKF_VariableKeyLerp;
	CompressedData.RotationCompressionFormat = ACF_Identity;
	CompressedData.TranslationCompressionFormat = ACF_Identity;
	CompressedData.ScaleCompressionFormat = ACF_Identity;
	CompressedData.BuildFinalBuffer(BulkData);

	FName AnimName;
	FFrameRate FrameRate;
	TArray<FTrackToSkeletonMap> SkeletonMap;

	GEngine->Exec(nullptr, *FormatCommand(true));
	{
		AEFVariableKeyLerp<ACF_Identity> KeyLerp;
		FAnimSequenceDecompressionContext DecompressContext(FrameRate,
			1,
			EAnimInterpolationType::Linear,
			AnimName,
			CompressedData,
			ISPCTransforms,
			SkeletonMap,
			nullptr,
			false,
			AAT_None);
		KeyLerp.GetPoseTranslations(ISPCAtoms, Bones, DecompressContext);
	}

	GEngine->Exec(nullptr, *FormatCommand(false));
	{
		AEFVariableKeyLerp<ACF_Identity> KeyLerp;
		FAnimSequenceDecompressionContext DecompressContext(FrameRate,
			1,
			EAnimInterpolationType::Linear,
			AnimName,
			CompressedData,
			CPPTransforms,
			SkeletonMap,
			nullptr,
			false,
			AAT_None);
		KeyLerp.GetPoseTranslations(CPPAtoms, Bones, DecompressContext);
	}

	GEngine->Exec(nullptr, *FormatCommand(InitialState));

	for (int32 i = 0; i < ISPCAtoms.Num(); ++i)
	{
		const FString Message(FString::Format(TEXT("Translation {0}"), { i }));
		TestTrue(*Message, ISPCAtoms[i].GetTranslation().Equals(CPPAtoms[i].GetTranslation()));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIspcTestAnimEncodingVariableKeyLerpGetPoseScales, "Ispc.Animation.VariableKeyLerp.GetPoseScales", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FIspcTestAnimEncodingVariableKeyLerpGetPoseScales::RunTest(const FString& Parameters)
{
	const FString CommandName("a.VariableKeyLerp.ISPC");
	auto FormatCommand = [CommandName](bool State) -> FString {
		return FString::Format(TEXT("{0} {1}"), { CommandName, State });
	};

	const IConsoleVariable* CVarISPCEnabled = IConsoleManager::Get().FindConsoleVariable(*CommandName);
	bool InitialState = CVarISPCEnabled->GetBool();
	check(GEngine);

	TArray<FTransform> ISPCTransforms;
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(45., 0., 0.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(45., 180., 0.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(1., -90., 0.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(1., 1., 100.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(0., 0., 0.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(1., 1., 1.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(360., 360., 360.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(9000., 9000., 9000.)));
	ISPCTransforms.Add(FTransform(FQuat4d(1., 1., 1., 1.), FVector3d(1., 1., 1.), FVector3d(-45., -180., 1.)));

	TArray<FTransform> CPPTransforms(ISPCTransforms);
	TArrayView<FTransform> ISPCAtoms(ISPCTransforms);
	TArrayView<FTransform> CPPAtoms(CPPTransforms);

	BoneTrackArray Bones;
	Bones.Add(BoneTrackPair(0, 0));
	Bones.Add(BoneTrackPair(1, 1));
	Bones.Add(BoneTrackPair(2, 2));

	TArray<uint8> BulkData;
	BulkData.Init(10, 100);
	FCompressedOffsetDataBase<TArray<int32>> ScaleOffets;
	ScaleOffets.OffsetData.Init(30, 100);

	FUECompressedAnimDataMutable CompressedData;
	CompressedData.CompressedTrackOffsets.Init(30, 100);
	CompressedData.CompressedScaleOffsets = ScaleOffets;
	CompressedData.CompressedByteStream = BulkData;
	CompressedData.KeyEncodingFormat = AKF_VariableKeyLerp;
	CompressedData.RotationCompressionFormat = ACF_Identity;
	CompressedData.TranslationCompressionFormat = ACF_Identity;
	CompressedData.ScaleCompressionFormat = ACF_Identity;
	CompressedData.BuildFinalBuffer(BulkData);

	FName AnimName;
	FFrameRate FrameRate;
	TArray<FTrackToSkeletonMap> SkeletonMap;

	GEngine->Exec(nullptr, *FormatCommand(true));
	{
		AEFVariableKeyLerp<ACF_Identity> KeyLerp;
		FAnimSequenceDecompressionContext DecompressContext(FrameRate,
			1,
			EAnimInterpolationType::Linear,
			AnimName,
			CompressedData,
			ISPCTransforms,
			SkeletonMap,
			nullptr,
			false,
			AAT_None);
		KeyLerp.GetPoseScales(ISPCAtoms, Bones, DecompressContext);
	}

	GEngine->Exec(nullptr, *FormatCommand(false));
	{
		AEFVariableKeyLerp<ACF_Identity> KeyLerp;
		FAnimSequenceDecompressionContext DecompressContext(FrameRate,
			1,
			EAnimInterpolationType::Linear,
			AnimName,
			CompressedData,
			CPPTransforms,
			SkeletonMap,
			nullptr,
			false,
			AAT_None);
		KeyLerp.GetPoseScales(CPPAtoms, Bones, DecompressContext);
	}

	GEngine->Exec(nullptr, *FormatCommand(InitialState));

	for (int32 i = 0; i < ISPCAtoms.Num(); ++i)
	{
		const FString Message(FString::Format(TEXT("Scale {0}"), { i }));
		TestTrue(*Message, ISPCAtoms[i].GetScale3D().Equals(CPPAtoms[i].GetScale3D()));
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
