// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MeshDeformerCollection.h"
#include "Animation/MeshDeformer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshDeformerCollection)

TArray<TSoftObjectPtr<UMeshDeformer>> UMeshDeformerCollection::GetMeshDeformers() const
{
	TSet<TObjectPtr<const UMeshDeformerCollection>> VisitedCollection;
	TArray<TSoftObjectPtr<UMeshDeformer>> Deformers;
	
	GetMeshDeformers_Internal(VisitedCollection, Deformers);
	
	return Deformers;
}

void UMeshDeformerCollection::GetMeshDeformers_Internal(
	TSet<TObjectPtr<const UMeshDeformerCollection>>& InOutVisitedCollection,
	TArray<TSoftObjectPtr<UMeshDeformer>>& InOutMeshDeformers
	) const
{
	if (InOutVisitedCollection.Contains(this))
	{
		return;
	}
	
	for (const TSoftObjectPtr<UMeshDeformer>& Deformer : MeshDeformers)
	{
		if (!Deformer.IsNull())
		{
			InOutMeshDeformers.AddUnique(Deformer);
		}
	}

	InOutVisitedCollection.Add(this);

	for (const TObjectPtr<UMeshDeformerCollection>& MeshDeformerCollection :  MeshDeformerCollections)
	{
		if (MeshDeformerCollection)
		{
			MeshDeformerCollection->GetMeshDeformers_Internal(InOutVisitedCollection, InOutMeshDeformers);
		}
	}
}
