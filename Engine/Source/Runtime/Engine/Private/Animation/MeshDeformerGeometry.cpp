// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MeshDeformerGeometry.h"

FMeshDeformerGeometry::FMeshDeformerGeometry()
{
}

void FMeshDeformerGeometry::Reset()
{
	PositionUpdatedFrame = TangentUpdatedFrame = ColorUpdatedFrame = 0;
	Position = PrevPosition = Tangent = Color = nullptr;
	PositionSRV = PrevPositionSRV = TangentSRV = ColorSRV = nullptr;
}
