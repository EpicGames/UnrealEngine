// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphResources.h"
#include "SkeletalRenderPublic.h"

/**
 * Owner class for Mesh Deformer generated geometry.
 */
class FMeshDeformerGeometry
{
public:
	FMeshDeformerGeometry();
	
	/** Reset the stored data. */
	void Reset();

	// Frame numbers of last update.
	uint32 PositionUpdatedFrame = 0;
	uint32 TangentUpdatedFrame = 0;
	uint32 ColorUpdatedFrame = 0;

	// Buffers containing deformed geometry data.
	TRefCountPtr<FRDGPooledBuffer> Position;
	TRefCountPtr<FRDGPooledBuffer> PrevPosition;
	TRefCountPtr<FRDGPooledBuffer> Tangent;
	TRefCountPtr<FRDGPooledBuffer> Color;
	// Shader resource views to the buffers.
	TRefCountPtr<FRHIShaderResourceView> PositionSRV;
	TRefCountPtr<FRHIShaderResourceView> PrevPositionSRV;
	TRefCountPtr<FRHIShaderResourceView> TangentSRV;
	TRefCountPtr<FRHIShaderResourceView> ColorSRV;
};
