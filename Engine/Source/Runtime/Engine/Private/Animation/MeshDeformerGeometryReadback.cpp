// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MeshDeformerGeometryReadback.h"
#include "MeshDescription.h"

FMeshDeformerGeometryReadbackRequest::~FMeshDeformerGeometryReadbackRequest()
{
	// Signal to the requester that the request was not successfully fulfilled
	
	if (MeshDescriptionCallback_AnyThread.IsSet())
	{
		if (!bMeshDescriptionHandled)
		{
			MeshDescriptionCallback_AnyThread(FMeshDescription());
		}
	}
	
	if (VertexDataArraysCallback_AnyThread.IsSet())
	{
		if (!bVertexDataArraysHandled)
		{
			VertexDataArraysCallback_AnyThread(FMeshDeformerGeometryReadbackVertexDataArrays());
		}
	}
}