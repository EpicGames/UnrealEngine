// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MirrorDataTable.h"

#include "AnimationRuntime.h"
#include "Algo/LevenshteinDistance.h"
#include "Animation/AnimationSettings.h"
#include "Animation/Skeleton.h"
#include "Internationalization/Regex.h"
#include "UObject/LinkerLoad.h"

#if WITH_EDITOR
#include "Misc/ScopedSlowTask.h"
#include "DataTableEditorUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Modules/ModuleManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MirrorDataTable)

#define LOCTEXT_NAMESPACE "MirrorDataTables"


FMirrorTableRow::FMirrorTableRow(const FMirrorTableRow& Other)
{
	*this = Other;
}

FMirrorTableRow& FMirrorTableRow::operator=(FMirrorTableRow const& Other)
{
	if (this == &Other)
	{
		return *this;
	}

	Name = Other.Name;
	MirroredName = Other.MirroredName;
	MirrorEntryType = Other.MirrorEntryType;
	bEnabled = Other.bEnabled;
	return *this;
}

bool FMirrorTableRow::operator==(FMirrorTableRow const& Other) const
{
	return (Name == Other.Name && MirroredName == Other.MirroredName && MirrorEntryType == Other.MirrorEntryType && bEnabled == Other.bEnabled);
}

bool FMirrorTableRow::operator!=(FMirrorTableRow const& Other) const
{
	return (Name != Other.Name || MirroredName != Other.MirroredName || MirrorEntryType != Other.MirrorEntryType || bEnabled != Other.bEnabled);
}

bool FMirrorTableRow::operator<(FMirrorTableRow const& Other) const
{
	if (MirrorEntryType == Other.MirrorEntryType)
	{
		if (Name != Other.Name)
		{
			return Name.LexicalLess(Other.Name);
		}
		else
		{
			return MirroredName.LexicalLess(Other.MirroredName);
		}
	}
	return MirrorEntryType < Other.MirrorEntryType;
}

UMirrorDataTable::UMirrorDataTable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), MirrorAxis(EAxis::X)
{
	
#if WITH_EDITOR
	OnDataTableChanged().AddUObject(this, &UMirrorDataTable::FillMirrorArrays);
#endif 
	
}

void UMirrorDataTable::PostLoad()
{
	Super::PostLoad();

	FillMirrorArrays();

#if WITH_EDITOR
	// Only warn for assets that have been synced before but are now stale.
	if (GetSkeletonSyncStatus() == ESyncStatus::Stale)
	{
		UE_LOGF(LogAnimation, Warning, "MirrorDataTable '%ls' is out of date with its skeleton. Consider reimporting from skeleton.", *GetName());
	}
#endif
}

#if WITH_EDITOR

void UMirrorDataTable::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UMirrorDataTable, Skeleton))
	{
		InvalidateCachedSkeletonData();
		FillMirrorArrays();
	}
	else if (MemberName != GET_MEMBER_NAME_CHECKED(UMirrorDataTable, bMirrorRootMotion))
	{
		// No need to update FillMirrorArrays() when bMirrorRootMotion changes as this is a variable used only at runtime when mirroring attributes.
		FillMirrorArrays();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

FName UMirrorDataTable::GetSettingsMirrorName(FName InName)
{
	UAnimationSettings* AnimationSettings = UAnimationSettings::Get();
	FName MirrorName;
	if (AnimationSettings)
	{
		MirrorName = GetMirrorName(InName, AnimationSettings->MirrorFindReplaceExpressions);
	}
	return MirrorName;
}

FName UMirrorDataTable::GetMirrorName(FName InName, const TArray<FMirrorFindReplaceExpression>& MirrorFindReplaceExpressions)
{
	FName ReplacedName = NAME_None;
	FString InNameString = InName.ToString();
	bool bFound = false;
	
	for (const FMirrorFindReplaceExpression& regExStr : MirrorFindReplaceExpressions)
	{
		FString FindString = regExStr.FindExpression.ToString();
		FString ReplaceString = regExStr.ReplaceExpression.ToString();

		switch (regExStr.FindReplaceMethod)
		{
			case EMirrorFindReplaceMethod::Prefix:

				if (InNameString.StartsWith(FindString, ESearchCase::CaseSensitive))
				{
					ReplaceString = ReplaceString + InNameString.Mid(FindString.Len());
					bFound = true;
				}
				break;
			
			case EMirrorFindReplaceMethod::Suffix:

				if (InNameString.EndsWith(FindString, ESearchCase::CaseSensitive))
				{
					ReplaceString = InNameString.LeftChop(FindString.Len()) + ReplaceString;
					bFound = true;
				}
				break;
			
			case EMirrorFindReplaceMethod::RegularExpression:

				FRegexPattern MatherPatter(FindString);
				FRegexMatcher Matcher(MatherPatter, InNameString);
				while (Matcher.FindNext())
				{
					for (int32 CaptureIndex = 1; CaptureIndex < 10; CaptureIndex++)
					{
						FString CaptureResult = Matcher.GetCaptureGroup(CaptureIndex);
						int32 CaptureBegin = Matcher.GetCaptureGroupBeginning(CaptureIndex);
						int32 CaptureEnd = Matcher.GetCaptureGroupEnding(CaptureIndex);
						FString CaptureRegion = CaptureResult.Mid(CaptureBegin, CaptureEnd - CaptureBegin);
						if (CaptureResult.IsEmpty())
						{
							break;
						}
						FString MatchString = FString::Printf(TEXT("$%i"), CaptureIndex);
						ReplaceString = ReplaceString.Replace(*MatchString, *CaptureResult);
					}
					bFound = true;
				}
				break;
		}

		// We found a match. Stop pattern matching.
		if (bFound)
		{
			ReplacedName = *ReplaceString;
			break;
		}
	}
	return ReplacedName;
}

FName UMirrorDataTable::FindReplace(FName InName) const
{
	return GetMirrorName(InName, MirrorFindReplaceExpressions); 
}

FName UMirrorDataTable::FindBestMirroredBone(
	const FName InBoneName,
	const FReferenceSkeleton& InRefSkeleton,
	EAxis::Type InMirrorAxis,
	const float SearchThreshold)
{
	const int32 SourceBoneIndex = InRefSkeleton.FindBoneIndex(InBoneName);
	if (!ensureMsgf(SourceBoneIndex != INDEX_NONE, TEXT("Trying to find a mirror for a bone that isn't in the skeleton.")))
	{
		return NAME_None;
	}
	
	// if the bone with the mirrored name exists in the skeleton, then great just use that...
	const FName MirroredName = GetSettingsMirrorName(InBoneName);
	if (InRefSkeleton.FindBoneIndex(MirroredName) != INDEX_NONE)
	{
		return MirroredName;
	}

	// fallback to closest mirrored bone, breaking ties (coincident bones) with fuzzy string score
	TArray<FTransform> RefPoseGlobal;
	FAnimationRuntime::FillUpComponentSpaceTransforms(InRefSkeleton, InRefSkeleton.GetRefBonePose(), RefPoseGlobal);
	FVector MirroredLocation = RefPoseGlobal[SourceBoneIndex].GetLocation();
	switch (InMirrorAxis)
	{
	case EAxis::X:
		MirroredLocation.X *= -1.f;
		break;
	case EAxis::Y:
		MirroredLocation.Y *= -1.f;
		break;
	case EAxis::Z:
		MirroredLocation.Z *= -1.f;
		break;
	default:
		checkNoEntry();
	}

	// find closest bone and all bones near the mirrored location within our search threshold
	TArray<int32> BonesWithinThreshold;
	int32 ClosestBone = 0;
	int32 CurrentBone = 0;
	float ClosestDistSq = TNumericLimits<float>::Max();
	for (const FTransform& BoneTransform : RefPoseGlobal)
	{
		const float DistSq = (BoneTransform.GetLocation() - MirroredLocation).SizeSquared();
		if (DistSq < ClosestDistSq)
		{
			ClosestDistSq = DistSq;
			ClosestBone = CurrentBone;
		}
		if (DistSq <= FMath::Pow(SearchThreshold, 2.f))
		{
			BonesWithinThreshold.Add(CurrentBone);
		}
		++CurrentBone;
	}

	// no other bones were found near the mirrored location, so return the closest one
	if (BonesWithinThreshold.Num() <= 1)
	{
		return InRefSkeleton.GetBoneName(ClosestBone);
	}
	
	// in the case where we have multiple bones at or near the mirrored location (that are within our search threshold)
	// it would be arbitrary to pick the "closest" one since bones are not always placed with that degree of precision.
	// in this case, we break the tie with a fuzzy string comparison with the source bone name...
	const FString SourceBoneStr = InBoneName.ToString().ToLower();
	float BestScore = 0.f;
	int32 BestBoneIndex = INDEX_NONE;
	for (const int32 BoneToTestIndex : BonesWithinThreshold)
	{
		FString BoneToTestStr = InRefSkeleton.GetBoneName(BoneToTestIndex).ToString().ToLower();
		const float WorstCase = SourceBoneStr.Len() + BoneToTestStr.Len();
		const float Score = 1.0f - (Algo::LevenshteinDistance(BoneToTestStr, SourceBoneStr) / WorstCase);
		if (Score > BestScore)
		{
			BestBoneIndex = BoneToTestIndex;
			BestScore = Score;
		}
	}
	
	return InRefSkeleton.GetBoneName(BestBoneIndex);
}

#if WITH_EDITOR  

UMirrorDataTable::FFindReplaceOptions UMirrorDataTable::FFindReplaceOptions::AddMissingOnly()
{
	return { EFlags::AddMissingRows, false };
}

UMirrorDataTable::FFindReplaceOptions UMirrorDataTable::FFindReplaceOptions::UpdateExisting()
{
	return { EFlags::UpdateExistingRows | EFlags::DisableStaleRows, false };
}

UMirrorDataTable::FFindReplaceOptions UMirrorDataTable::FFindReplaceOptions::Sync()
{
	return { EFlags::UpdateExistingRows | EFlags::AddMissingRows | EFlags::DisableStaleRows, false };
}

UMirrorDataTable::FFindReplaceOptions& UMirrorDataTable::FFindReplaceOptions::WithNotification(bool bInShowNotification)
{
	bShowNotification = bInShowNotification;
	return *this;
}

void UMirrorDataTable::FindReplaceMirroredNames()
{
	UpdateFromFindReplaceExpressions(FFindReplaceOptions::AddMissingOnly());
}

void UMirrorDataTable::UpdateFromFindReplaceExpressions(const FFindReplaceOptions& Options)
{
	if (!Skeleton)
	{
		return; 
	}

	// @todo: LogAnimation default filter is Warning so any Log messages will not show in the UE unless `log Animation Log` command is ran. Not useful for TDiAs.
	UE_LOGF(LogAnimation, Log, "Starting operation to update MirrorDataTable (%ls) via Find Replace expressions", *GetName());
	
	// Determine what are we doing.
	const bool bShouldUpdateExisting = EnumHasAnyFlags(Options.Flags, FFindReplaceOptions::EFlags::UpdateExistingRows);
	const bool bShouldAddMissing = EnumHasAnyFlags(Options.Flags, FFindReplaceOptions::EFlags::AddMissingRows);
	const bool bShouldDisableStale = EnumHasAnyFlags(Options.Flags, FFindReplaceOptions::EFlags::DisableStaleRows);
	
	// Keep track of progress.
	constexpr int32 NumProgressSteps = 7; // Gather entries: 1, Update/Add each entry type: 2-5, Disable missing entries: 6, Update runtime maps: 7. 
	FScopedSlowTask Progress(NumProgressSteps, LOCTEXT("FindReplaceMirroredNamesOperation", "Finding and/or replacing mirror pairs..."));
	
	// Show progress dialog.
	constexpr bool bShowCancelButton = true;
	Progress.MakeDialog(bShowCancelButton);
	
	// Build map of existing entries (organized by their row type).
	TStaticArray<FCategoryState, 5> CategoryStates;
	InitCategoryEntryStates(CategoryStates);
	
	bool bChangedTable = false;	// Has any change was done to the table?
	bool bModified = false;		// Has this table already been modified?
	
	// Operation stats.
	int NumAddedRows = 0;
	int NumUpdatedRows = 0;
	int NumStaleRows = 0;
	int NumDuplicateRows = 0;
		
	// Call any time we attempt to modify the inner data table struct.
	auto TryToModify = [this, &bModified, &bChangedTable]()
	{
		// Ensure we only modify once to avoid storing extra data in transaction buffer.
		if (!bModified)
		{
			Modify();
			bModified = true;
		}
			
		// Any time we try to modify it means we changed the internal data of the table.
		bChangedTable = true;
	};
	
	// Finds or adds a new mirror row (ensuring it has a unique name).
	auto FindOrAddMirrorRow = [this, &TryToModify, &CategoryStates, &NumAddedRows](const FName& EntryName, const EMirrorRowType::Type RowType, const bool bAllowAdd) -> FMirrorTableRow*
	{
		FCategoryState& State = CategoryStates[RowType];
			
		// Get the actual row name in the data table.
		if (const FName* ExistingRowName = State.EntryNameToRowName.Find(EntryName))
		{
			return FindRow<FMirrorTableRow>(*ExistingRowName, TEXT("UMirrorDataTable::FindOrAddMirrorRow"), false);
		}
		
		// We couldn't find the row, but we aren't able to add a new one, exit.
		if (!bAllowAdd || !RowStruct)
		{
			return nullptr;
		}
			
		// Get unique row name across data table entries.
		const FName NewRowName = MakeUniqueMirrorRowName(EntryName, RowType);
			
		// Add new row to data table.
		{
			TryToModify();
			
			// Allocate data to store information, using UScriptStruct to know its size
			uint8* RowData = (uint8*)FMemory::Malloc(RowStruct->GetStructureSize());
			RowStruct->InitializeStruct(RowData);
			
			// Directly add rows to avoid using FDataTableEditorUtils, which is not appropriate at this point
			// equivalent to FDataTableEditorUtils::AddRow(DataTable, BoneName). This is so we can batch changes and only fire notifications at the end. 
			AddRowInternal(NewRowName, RowData);
			
			++NumAddedRows;
			
			UE_LOGF(LogAnimation, Log, "MirrorDataTable '%ls': Added new row '%ls'", *GetName(), *NewRowName.ToString());
		}
			
		// Keep the cached state accurate for the rest of this run.
		State.ExistingEntryNames.Add(EntryName);
		State.EntryNameToRowName.Add(EntryName, NewRowName);
			
		return FindRow<FMirrorTableRow>(NewRowName, TEXT("UMirrorDataTable::FindOrAddMirrorRow"), false);
	};
	
	// Updates or add a mirror row (if entry is missing from the data table). Only calls Modify() at most once.
	auto UpdateOrAddMirrorRow = [this,  &TryToModify, &CategoryStates, &FindOrAddMirrorRow, &NumUpdatedRows, bShouldUpdateExisting, bShouldAddMissing]
	(const FName& EntryName, const FName& MirroredName, EMirrorRowType::Type RowType, bool bEnabled)
	{
		FCategoryState& State = CategoryStates[RowType];
		
		// Ambiguous table state. Skip auto-update one of multiple rows.
		if (State.DuplicateEntryNames.Contains(EntryName))
		{
			UE_LOGF(LogAnimation, Warning, "MirrorDataTable '%ls': Ambiguous table state. Skipping auto-update of entry '%ls' due to it having duplicates.", *GetName(), *EntryName.ToString());
			return;
		}
			
		// Mark this entry as present in the current skeleton scan.
		State.ProcessedEntryNames.Add(EntryName);
			
		const bool bHasExistingEntry = State.ExistingEntryNames.Contains(EntryName);
			
		FMirrorTableRow* MirrorRow = nullptr;
			
		if (bHasExistingEntry)
		{
			// Reliable lookup using actual row name
			FName ExistingRowName = State.EntryNameToRowName.FindRef(EntryName);
			MirrorRow = FindRow<FMirrorTableRow>(ExistingRowName, TEXT("UMirrorDataTable::UpdateOrAddMirrorRow"), false);
			
			
			// Update values.
			if (MirrorRow && bShouldUpdateExisting)
			{
				TryToModify();
				
				const FMirrorTableRow UpdatedRow = { EntryName, MirroredName, RowType, bEnabled };
				const bool bHasAnythingChanged = UpdatedRow != *MirrorRow;
				
				*MirrorRow = UpdatedRow;

				if (bHasAnythingChanged)
				{
					++NumUpdatedRows;
				}
				
				UE_LOGF(LogAnimation, Log, "MirrorDataTable '%ls': Updated data for row '%ls'.", *GetName(),  *ExistingRowName.ToString());
			}
		}
		else
		{
			MirrorRow = FindOrAddMirrorRow(EntryName, RowType, bShouldAddMissing);
			if (MirrorRow)
			{
				// TryToModify() is already called by FindOrAddMirrorRow().
				
				MirrorRow->Name = EntryName;
				MirrorRow->MirroredName = MirroredName;
				MirrorRow->MirrorEntryType = RowType;
				MirrorRow->bEnabled = bEnabled;
			}
		}
	};
	
	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	
	// Helper to get mirrored entry name.
	UAnimationSettings* AnimationSettings = UAnimationSettings::Get();
	auto GetMirroredName = [&AnimationSettings, this](const FName& InName) -> FName
	{
		FName MirroredName = FindReplace(InName);
			
		if (MirroredName.IsNone())
		{
			if (AnimationSettings && AnimationSettings->bOnMirrorPairFindReplaceFailedMapToSelf)
			{
				MirroredName = InName;
				UE_LOGF(LogAnimation, Warning, "MirrorDataTable '%ls': No mirror entry found for '%ls', falling back to self.", *GetName(), *InName.ToString());
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "MirrorDataTable '%ls': No mirror entry found for '%ls'.", *GetName(), *InName.ToString());
			}
		}
		return MirroredName;
	};
	
	// Add mirrored bone pairs.
	Progress.EnterProgressFrame(1.0f, LOCTEXT("FindReplaceMirroredNamesOperation_AddingBones", "Adding bone mirror pair entries..."));
	if (BoneScope == EMirrorTableBoneRefreshScope::ExplicitBoneList && BoneScopeNameList.Num() > 0)
	{
		for (const FName& BoneName : BoneScopeNameList)
		{
			if (RefSkeleton.FindBoneIndex(BoneName) == INDEX_NONE)
			{
				continue;
			}
			const FName MirroredName = GetMirroredName(BoneName);
			const bool bEnabled = !MirroredName.IsNone() && RefSkeleton.FindBoneIndex(MirroredName) != INDEX_NONE;
			UpdateOrAddMirrorRow(BoneName, MirroredName, EMirrorRowType::Bone, bEnabled);
		}
	}
	else
	{
		for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
		{
			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
			const FName MirroredName = GetMirroredName(BoneName);
			const bool bEnabled = !MirroredName.IsNone() && RefSkeleton.FindBoneIndex(MirroredName) != INDEX_NONE;
			UpdateOrAddMirrorRow(BoneName, MirroredName, EMirrorRowType::Bone, bEnabled);
		}
	}
	
	// Add mirrored animation notify pairs.
	Progress.EnterProgressFrame(1.0f, LOCTEXT("FindReplaceMirroredNamesOperation_AddingNotifies", "Adding animation notify mirror pair entries..."));
	for (const FName& Notify : Skeleton->AnimationNotifies)
	{
		const FName MirroredName = GetMirroredName(Notify);
		const bool bEnabled = !MirroredName.IsNone() && Skeleton->AnimationNotifies.Contains(MirroredName);
		
		UpdateOrAddMirrorRow(Notify, MirroredName, EMirrorRowType::AnimationNotify, bEnabled);
	}
	
	// Add mirrored sync markers pairs.
	Progress.EnterProgressFrame(1.0f, LOCTEXT("FindReplaceMirroredNamesOperation_AddingSyncMarkers", "Adding sync marker mirror pair entries..."));
	for (const FName& SyncMarker : Skeleton->GetExistingMarkerNames())
	{
		const FName MirroredName = GetMirroredName(SyncMarker);
		const bool bEnabled = !MirroredName.IsNone() && Skeleton->GetExistingMarkerNames().Contains(MirroredName);
		
		UpdateOrAddMirrorRow(SyncMarker, MirroredName, EMirrorRowType::SyncMarker, bEnabled);
	}

	// Add mirrored curves pairs.
	Progress.EnterProgressFrame(1.0f, LOCTEXT("FindReplaceMirroredNamesOperation_AddingCurves", "Adding curve mirror pair entries..."));
	{
		TArray<FName> CurveNames;
		Skeleton->GetCurveMetaDataNames(CurveNames);
		
		for (const FName& CurveName : CurveNames)
		{
			const FName MirroredName = GetMirroredName(CurveName);
			const bool bEnabled = !MirroredName.IsNone() && CurveNames.Contains(MirroredName);

			UpdateOrAddMirrorRow(CurveName, MirroredName, EMirrorRowType::Curve, bEnabled);
		}
	}
	
	// @todo: Disable rows with no entry name or mirror entry name.
	
	// Disable stale skeleton entries.
	Progress.EnterProgressFrame(1.0f, LOCTEXT("FindReplaceMirroredNamesOperation_DisabledEntriesMissingFromSkeleton", "Disabling skeleton based entries missing from the associated skeleton ..."));
	if (bShouldDisableStale)
	{
		// Exclude "Custom" category since those entries do not come from the skeleton.
		constexpr int CategoriesToCheck = (CategoryStates.Num() - 1);
		
		for (int CategoryIndex = 0; CategoryIndex < CategoriesToCheck; ++CategoryIndex)
		{
			FCategoryState& State = CategoryStates[CategoryIndex];
			
			// Entries that existed in the data table but were not encountered from the current skeleton scan.
			TSet<FName> StaleEntryNames = State.ExistingEntryNames.Difference(State.ProcessedEntryNames);
			
			for (const FName& EntryName : StaleEntryNames)
			{
				// Ambiguous table state. Skip auto-update one of multiple rows.
				if (State.DuplicateEntryNames.Contains(EntryName))
				{
					continue;
				}
				
				// Reliable lookup using actual row name.
				const FName* StaleRowName = State.EntryNameToRowName.Find(EntryName);
				if (!StaleRowName)
				{
					continue;
				}
				
				// Get stale row.
				FMirrorTableRow* StaleMirrorRow = FindRow<FMirrorTableRow>(*StaleRowName, TEXT("UMirrorDataTable::ApplyFindReplaceExpressions"), false);
				if (!StaleMirrorRow)
				{
					continue;
				}
				
				// Mark as stale, if not already.
				if (StaleMirrorRow->bEnabled)
				{
					TryToModify();
					
					StaleMirrorRow->bEnabled = false;
					
					++NumStaleRows;
					
					UE_LOGF(LogAnimation, Log, "MirrorDataTable '%ls': Disabled stale row '%ls'.", *GetName(), *StaleRowName->ToString());
				}
			}
		}
	}
	
	// Check if we have any duplicate entries at all.
	for (int CategoryIndex = 0; CategoryIndex < CategoryStates.Num(); ++CategoryIndex)
	{
		NumDuplicateRows += CategoryStates[CategoryIndex].DuplicateEntryNames.Num();
	}
	
	// Keep tracks that were up-to-date now.
	const bool bIsFullyRefreshed = bShouldUpdateExisting && bShouldAddMissing && bShouldDisableStale;
	const bool bAreAnyWarningsPresent = NumDuplicateRows > 0;
	if (bIsFullyRefreshed && !bAreAnyWarningsPresent)
	{
		SkeletonHierarchyGuid = Skeleton->GetGuid();
		SkeletonVirtualBonesHierarchyGuid = Skeleton->GetVirtualBoneGuid();
	}
	
	// Let others react to data table changes.
	Progress.EnterProgressFrame(1.0f, LOCTEXT("FindReplaceMirroredNamesOperation_UpdatingRuntimeData", "Updating runtime mirror maps..."));
	if (bChangedTable)
	{
		OnDataTableChanged().Broadcast();
		
		// Note: There is no need to run FillMirrorArrays(), as this will be handled via broadcasting the OnDataTableChanged delegate.
		
		// Let editor know we've changed the data and refresh view.
		FDataTableEditorUtils::BroadcastPostChange(this, FDataTableEditorUtils::EDataTableChangeInfo::RowList);
	}
	
	// Show completion notification from operation.
	if (Options.bShowNotification)
	{
		const bool bFailure = bAreAnyWarningsPresent;
		const FText FormatFailure = LOCTEXT("MirrorReimportSummary_Failure", "Update complete with warnings. See log for Details. Skipped auto-update for {3} duplicate entries. Added {0} entries, updated {1} entries, disabled {2} stale entries."); 
		const FText FormatSuccess = LOCTEXT("MirrorReimportSummary_Success", "Update complete. See log for details. Added {0} entries, updated {1} entries, disabled {2} stale entries.");
		const FText Message = FText::Format(
			bFailure ? FormatFailure : FormatSuccess,
			NumAddedRows,
			NumUpdatedRows,
			NumStaleRows,
			NumDuplicateRows
		);
		
		FNotificationInfo Info(Message);
		Info.bFireAndForget = true;
		Info.ExpireDuration = 7.0f;
		Info.bUseSuccessFailIcons = true;
		
		if (TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info))
		{
			Notification->SetCompletionState(bFailure ? SNotificationItem::CS_Fail: SNotificationItem::CS_Success);
		}
	}
}

UMirrorDataTable::ESyncStatus UMirrorDataTable::GetSkeletonSyncStatus() const
{
	if (!Skeleton)
	{
		return ESyncStatus::Unknown;
	}

	if (!SkeletonHierarchyGuid.IsValid())
	{
		return ESyncStatus::Unknown;
	}

	const bool bIsUpToDate = Skeleton->GetGuid() == SkeletonHierarchyGuid && Skeleton->GetVirtualBoneGuid() == SkeletonVirtualBonesHierarchyGuid;

	return bIsUpToDate ? ESyncStatus::UpToDate : ESyncStatus::Stale;
}

void UMirrorDataTable::InvalidateCachedSkeletonData()
{
	// Non-zero sentinel: Will be valid but won't match any skeleton, so GetSkeletonSyncStatus() returns Stale (not Unknown).
	static constexpr FGuid StaleGuid(MAX_uint32, MAX_uint32, MAX_uint32, MAX_uint32);
	
	SkeletonHierarchyGuid = StaleGuid;
	SkeletonVirtualBonesHierarchyGuid = StaleGuid;
}

#endif // WITH_EDITOR

void UMirrorDataTable::FillCompactPoseMirrorBones(const FBoneContainer& BoneContainer, const TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex>& MirrorBoneIndexes, TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex>& OutCompactPoseMirrorBones)
{
	const int32 NumReqBones = BoneContainer.GetCompactPoseNumBones();
	OutCompactPoseMirrorBones.Reset(NumReqBones);

	if (MirrorBoneIndexes.Num() > 0)
	{
		for (int32 CompactBoneIndex = 0; CompactBoneIndex < NumReqBones; ++CompactBoneIndex)
		{
			FSkeletonPoseBoneIndex SkeletonPoseBoneIndex = BoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(FCompactPoseBoneIndex(CompactBoneIndex));

			//Mirror Bone
			const FSkeletonPoseBoneIndex MirrorIndex = MirrorBoneIndexes.IsValidIndex(SkeletonPoseBoneIndex.GetInt()) ? MirrorBoneIndexes[SkeletonPoseBoneIndex] : FSkeletonPoseBoneIndex(INDEX_NONE);

			OutCompactPoseMirrorBones.Add(BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(MirrorIndex));
		}
	}
	else
	{
		for (int32 CompactBoneIndex = 0; CompactBoneIndex < NumReqBones; ++CompactBoneIndex)
		{
			OutCompactPoseMirrorBones.Add(FCompactPoseBoneIndex(INDEX_NONE));
		}
	}
}

void UMirrorDataTable::FillMirrorBoneIndexes(const USkeleton* InSkeleton, TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex>& OutMirrorBoneIndexes) const
{
	const FReferenceSkeleton& ReferenceSkeleton = InSkeleton->GetReferenceSkeleton();

	// Reset the mirror table to defaults (no mirroring)
	OutMirrorBoneIndexes.SetNumUninitialized(ReferenceSkeleton.GetNum());
	FMemory::Memset(OutMirrorBoneIndexes.GetData(), INDEX_NONE, OutMirrorBoneIndexes.Num() * OutMirrorBoneIndexes.GetTypeSize());

	TMap<FName, FName> NameToMirrorNameBoneMap; 
	ForeachRow<FMirrorTableRow>(TEXT("UMirrorDataTable::FillMirrorBoneIndexes"), [&NameToMirrorNameBoneMap](const FName& Key, const FMirrorTableRow& Value) mutable
		{
			if (Value.MirrorEntryType == EMirrorRowType::Bone && Value.bEnabled)
			{
				NameToMirrorNameBoneMap.Add(Value.Name, Value.MirroredName);
			}
		}
	);

	if (MirrorAxis != EAxis::None)
	{
		for (int32 BoneIndex = 0; BoneIndex < OutMirrorBoneIndexes.Num(); ++BoneIndex)
		{
			if (!OutMirrorBoneIndexes[BoneIndex].IsValid())
			{
				// Find the candidate mirror partner for this bone (falling back to mirroring to self)
				FName SourceBoneName = ReferenceSkeleton.GetBoneName(BoneIndex);
				int32 MirrorBoneIndex = INDEX_NONE;

				FName* MirroredBoneName = NameToMirrorNameBoneMap.Find(SourceBoneName);
				if (!SourceBoneName.IsNone() && MirroredBoneName)
				{
					MirrorBoneIndex = ReferenceSkeleton.FindBoneIndex(*MirroredBoneName);
				}

				OutMirrorBoneIndexes[BoneIndex] = FSkeletonPoseBoneIndex(MirrorBoneIndex);
				if (MirrorBoneIndex != INDEX_NONE)
				{
					OutMirrorBoneIndexes[MirrorBoneIndex] = FSkeletonPoseBoneIndex(BoneIndex);
				}
			}
		}
	}
}

void UMirrorDataTable::FillCompactPoseAndComponentRefRotations(
	const FBoneContainer& BoneContainer,
	TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex>& OutCompactPoseMirrorBones,
	TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex>& OutComponentSpaceRefRotations) const
{
	TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex> MirrorBoneIndexes;
	FillMirrorBoneIndexes(BoneContainer.GetSkeletonAsset(), MirrorBoneIndexes);
	FillCompactPoseMirrorBones(BoneContainer, MirrorBoneIndexes, OutCompactPoseMirrorBones);

	const int32 NumBones = BoneContainer.GetCompactPoseNumBones();
	OutComponentSpaceRefRotations.SetNumUninitialized(NumBones);
	if (NumBones > 0 && BoneContainer.GetRefPoseArray().Num() > 0)
	{
		OutComponentSpaceRefRotations[FCompactPoseBoneIndex(0)] =
			BoneContainer.GetRefPoseTransform(FCompactPoseBoneIndex(0)).GetRotation();
	}
	for (FCompactPoseBoneIndex BoneIndex(1); BoneIndex < NumBones; ++BoneIndex)
	{
		const FCompactPoseBoneIndex ParentBoneIndex = BoneContainer.GetParentBoneIndex(BoneIndex);
		OutComponentSpaceRefRotations[BoneIndex] = 
			OutComponentSpaceRefRotations[ParentBoneIndex] * BoneContainer.GetRefPoseTransform(BoneIndex).GetRotation();
	}
}

void UMirrorDataTable::FillMirrorArrays()
{
	DisabledEntries.Empty();
	SyncToMirrorSyncMap.Empty();
	AnimNotifyToMirrorAnimNotifyMap.Empty();
	CurveToMirrorCurveMap.Empty();
	
	if (!Skeleton)
	{
		BoneToMirrorBoneIndex.Empty();
		return; 
	}

	FillMirrorBoneIndexes(Skeleton, BoneToMirrorBoneIndex);
	
	ForeachRow<FMirrorTableRow>(TEXT("UMirrorDataTable::FillMirrorArrays"), [this](const FName& Key, const FMirrorTableRow& Value) mutable
		{
			if (!Value.bEnabled)
			{
				DisabledEntries.Add(Value.MirroredName);
				return;
			}
			
			switch (Value.MirrorEntryType)
			{
			case  EMirrorRowType::Curve:
			{
				// curves swap, so only one entry should exist.  For instance, if the table has (Left, Right) and (Right, Left) only add one item
				const FName* TestMirroredMatch = CurveToMirrorCurveMap.Find(Value.MirroredName);
				if (TestMirroredMatch == nullptr || *TestMirroredMatch != Value.Name)
				{
					CurveToMirrorCurveMap.Add(Value.Name, Value.MirroredName);
				}
				break;
			}
			case EMirrorRowType::SyncMarker:
			{
                SyncToMirrorSyncMap.Add(Value.Name, Value.MirroredName);
				break;
            }
			case EMirrorRowType::AnimationNotify:
			{
                AnimNotifyToMirrorAnimNotifyMap.Add(Value.Name, Value.MirroredName);
				break;
            }
			default:
				break;
			}
		}
	);
}

void UMirrorDataTable::InitCategoryEntryStates(TStaticArray<FCategoryState, 5>& OutStates) const
{
	static FString CategoryNames[] = { "Bone", "AnimationNotify", "Curve", "SyncMarker", "Custom" };
	
	ForeachRow<FMirrorTableRow>(TEXT("UMirrorDataTable::BuildCategoryEntryState"), [&OutStates](const FName& RowName, const FMirrorTableRow& Value)
		{
			const int32 CategoryIndex = Value.MirrorEntryType;
			check(CategoryIndex >= 0 && CategoryIndex < OutStates.Num())
			
			FCategoryState & State = OutStates[CategoryIndex];

			// We can assume that bones, notifies, sync markers, and curves are uniquely named.
			// However, if any matching row name (of the same type) was manually added this can lead to duplicates, in this we just skip updating them and inform user.
			if (const FName* ExistingRowName = State.EntryNameToRowName.Find(Value.Name))
			{
				UE_LOGF(LogAnimation, Warning, "Duplicate mirror table entry `%ls` found in category `%ls`. Existing row `%ls`, duplicate row `%ls`.", *Value.Name.ToString(), *CategoryNames[CategoryIndex], *ExistingRowName->ToString(), *RowName.ToString())
				State.DuplicateEntryNames.Add(Value.Name);
				return;
			}
			
			State.ExistingEntryNames.Add(Value.Name);
			State.EntryNameToRowName.Add(Value.Name, RowName);
		}
	);
}

FName UMirrorDataTable::MakeUniqueMirrorRowName(const FName EntryName, const EMirrorRowType::Type RowType) const
{
	static const FString CategorySuffix[] = { TEXT(":Bone"), TEXT(":Notify"), TEXT(":Curve"), TEXT(":SyncMarker"), TEXT(":Custom") };
	
	FName CandidateRowName = EntryName;
	uint32 RenamedAttempts = 0;
	
	while (FindRow<FMirrorTableRow>(CandidateRowName, TEXT("UMirrorDataTable::MakeUniqueMirrorRowName"), false))
	{
		FString RowString = EntryName.ToString() + CategorySuffix[RowType];
		if (RenamedAttempts > 0)
		{
			RowString.Appendf(TEXT("%u"), RenamedAttempts);
		}
		
		CandidateRowName = FName(*RowString);
		++RenamedAttempts;
	}
	
	return CandidateRowName;
}

#undef LOCTEXT_NAMESPACE
