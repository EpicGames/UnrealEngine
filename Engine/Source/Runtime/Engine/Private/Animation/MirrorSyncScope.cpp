// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MirrorSyncScope.h"
#include "Animation/AnimSyncScope.h"
#include "Animation/AnimNodeBase.h"

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::Anim::FMirrorSyncScope);

IMPLEMENT_NOTIFY_CONTEXT_INTERFACE(UE::Anim::FAnimNotifyMirrorContext)

namespace UE { namespace Anim {

bool IsMirrorSyncIdentical(const UMirrorDataTable* MirrorTableA, const UMirrorDataTable* MirrorTableB)
{
	if (MirrorTableA == MirrorTableB)
	{
		return true; 
	}
	if (MirrorTableA->SyncToMirrorSyncMap.Num() == MirrorTableB->SyncToMirrorSyncMap.Num())
	{
		for (auto& Elem : MirrorTableA->SyncToMirrorSyncMap)
		{
			const FName* BValue = MirrorTableB->SyncToMirrorSyncMap.Find(Elem.Key);
			if( BValue == nullptr || *BValue != Elem.Value)
			{
				return false; 
			}
		}
		return true; 
	}
	return false; 
}

FMirrorSyncScope::FMirrorSyncScope(const FAnimationBaseContext& InContext, const UMirrorDataTable* InMirrorDataTable)
	: MirrorDataTable(InMirrorDataTable)
{
	bool bClearMirrorSync = false; 
	if (FMirrorSyncScope* ParentMirrorScope = InContext.GetMessage<FMirrorSyncScope>())
	{
		// Syncing is done when animation sequences are evaluated, which means we need to examine the stack of mirror
		// nodes to determine if we can simply enable and disable sync mirroring.  
		MirrorScopeDepth = ParentMirrorScope->GetMirrorScopeDepth() + 1;
		OuterScopeMirrorDataTable = ParentMirrorScope->MirrorDataTable;
		if (MirrorScopeDepth % 2 == 0)
		{
			bClearMirrorSync = IsMirrorSyncIdentical(ParentMirrorScope->MirrorDataTable, MirrorDataTable);
		}
	}

	if (InContext.SharedContext)
	{
		TWeakPtr<FAnimSyncGroupScope> SyncScope;
		InContext.SharedContext->MessageStack.TopMessageWeakPtr<FAnimSyncGroupScope>([&SyncScope](TWeakPtr<FAnimSyncGroupScope>& InMessage)
			{
				SyncScope = InMessage;
			});
		AnimSyncGroupScope = SyncScope; 
	}

	if(TSharedPtr<FAnimSyncGroupScope> PinnedSyncGroupScope = AnimSyncGroupScope.Pin())
	{
		if (bClearMirrorSync)
		{
			MirrorDataTable = nullptr;
		}
		PinnedSyncGroupScope.Get()->SetMirror(MirrorDataTable);
	}
}

FMirrorSyncScope::~FMirrorSyncScope()
{
	if (TSharedPtr<FAnimSyncGroupScope> PinnedSyncGroupScope = AnimSyncGroupScope.Pin())
	{
		PinnedSyncGroupScope->SetMirror(OuterScopeMirrorDataTable);
	}
}

int32 FMirrorSyncScope::GetMirrorScopeDepth() const
{
	return MirrorScopeDepth;
}

TUniquePtr<const IAnimNotifyEventContextDataInterface> FMirrorSyncScope::MakeUniqueEventContextData() const
{
	return MakeUnique<FAnimNotifyMirrorContext>(MirrorDataTable);
}

}}	// namespace UE::Anim
