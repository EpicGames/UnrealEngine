// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MotionTrajectoryTypes.h"
#include "Algo/AllOf.h"
#include "Animation/AnimTypes.h"
#include "DrawDebugHelpers.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTypes.h"
#include "HAL/IConsoleManager.h"
#include "Misc/StringFormatArg.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionTrajectoryTypes)

#if ENABLE_ANIM_DEBUG
static constexpr int32 DebugTrajectorySampleDisable = 0;
static constexpr int32 DebugTrajectorySampleCount = 1;
static constexpr int32 DebugTrajectorySampleTime = 2;
static constexpr int32 DebugTrajectorySamplePosition = 3;
static constexpr int32 DebugTrajectorySampleVelocity = 4;
static const FVector DebugSampleTypeOffset(0.f, 0.f, 50.f);
static const FVector DebugSampleOffset(0.f, 0.f, 10.f);

TAutoConsoleVariable<int32> CVarMotionTrajectoryDebug(TEXT("a.MotionTrajectory.Debug"), 0, TEXT("Turn on debug drawing for motion trajectory"));
TAutoConsoleVariable<int32> CVarMotionTrajectoryDebugStride(TEXT("a.MotionTrajectory.Stride"), 1, TEXT("Configure the sample stride when displaying information"));
TAutoConsoleVariable<int32> CVarMotionTrajectoryDebugOptions(TEXT("a.MotionTrajectory.Options"), 0, TEXT("Toggle motion trajectory sample information:\n 0. Disable Text\n 1. Index\n2. Accumulated Time\n 3. Position\n 4. Velocity\n 5. Acceleration"));
#endif

namespace
{
	template<class U> static inline U CubicCRSplineInterpSafe(const U& P0, const U& P1, const U& P2, const U& P3, const float I, const float A = 0.5f)
	{
		float D1;
		float D2;
		float D3;

		if constexpr (TIsFloatingPoint<U>::Value)
		{
			D1 = FMath::Abs(P1 - P0);
			D2 = FMath::Abs(P2 - P1);
			D3 = FMath::Abs(P3 - P2);
		}
		else
		{
			D1 = static_cast<float>(FVector::Distance(P0, P1));
			D2 = static_cast<float>(FVector::Distance(P2, P1));
			D3 = static_cast<float>(FVector::Distance(P3, P2));
		}

		const float T0 = 0.f;
		const float T1 = T0 + FMath::Pow(D1, A);
		const float T2 = T1 + FMath::Pow(D2, A);
		const float T3 = T2 + FMath::Pow(D3, A);

		return FMath::CubicCRSplineInterpSafe(P0, P1, P2, P3, T0, T1, T2, T3, FMath::Lerp(T1, T2, I));
	}
}

bool FTrajectorySample::IsZeroSample() const
{
	// AccumulatedTime is specifically omitted here to allow for the zero sample semantic across an entire trajectory range
	return LinearVelocity.IsNearlyZero()
		&& Transform.GetTranslation().IsNearlyZero()
		&& Transform.GetRotation().IsIdentity();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FTrajectorySample FTrajectorySample::Lerp(const FTrajectorySample& Sample, float Alpha) const
{
	FTrajectorySample Interp;
	Interp.AccumulatedSeconds = FMath::Lerp(AccumulatedSeconds, Sample.AccumulatedSeconds, Alpha);
	Interp.LinearVelocity = FMath::Lerp(LinearVelocity, Sample.LinearVelocity, Alpha);

	Interp.Transform.Blend(Transform, Sample.Transform, Alpha);
	
	return Interp;
}

FTrajectorySample FTrajectorySample::SmoothInterp(const FTrajectorySample& PrevSample
	, const FTrajectorySample& Sample
	, const FTrajectorySample& NextSample
	, float Alpha) const
{
	FTrajectorySample Interp;
	Interp.AccumulatedSeconds = CubicCRSplineInterpSafe(PrevSample.AccumulatedSeconds, AccumulatedSeconds, Sample.AccumulatedSeconds, NextSample.AccumulatedSeconds, Alpha);
	Interp.LinearVelocity = CubicCRSplineInterpSafe(PrevSample.LinearVelocity, LinearVelocity, Sample.LinearVelocity, NextSample.LinearVelocity, Alpha);

	Interp.Transform.SetLocation(CubicCRSplineInterpSafe(
		PrevSample.Transform.GetLocation(),
		Transform.GetLocation(),
		Sample.Transform.GetLocation(),
		NextSample.Transform.GetLocation(),
		Alpha));
	FQuat Q0 = PrevSample.Transform.GetRotation().W >= 0.0f ? 
		PrevSample.Transform.GetRotation() : -PrevSample.Transform.GetRotation();
	FQuat Q1 = Transform.GetRotation().W >= 0.0f ? 
		Transform.GetRotation() : -Transform.GetRotation();
	FQuat Q2 = Sample.Transform.GetRotation().W >= 0.0f ? 
		Sample.Transform.GetRotation() : -Sample.Transform.GetRotation();
	FQuat Q3 = NextSample.Transform.GetRotation().W >= 0.0f ? 
		NextSample.Transform.GetRotation() : -NextSample.Transform.GetRotation();

	FQuat T0, T1;
	FQuat::CalcTangents(Q0, Q1, Q2, 0.0f, T0);
	FQuat::CalcTangents(Q1, Q2, Q3, 0.0f, T1);

	Interp.Transform.SetRotation(FQuat::Squad(Q1, T0, Q2, T1, Alpha));

	return Interp;
}

void FTrajectorySample::PrependOffset(const FTransform DeltaTransform, float DeltaSeconds)
{
	AccumulatedSeconds += DeltaSeconds;
	Transform *= DeltaTransform;
	LinearVelocity = DeltaTransform.TransformVectorNoScale(LinearVelocity);
}

void FTrajectorySample::TransformReferenceFrame(const FTransform DeltaTransform)
{
	Transform = DeltaTransform.Inverse() * Transform * DeltaTransform;
	LinearVelocity = DeltaTransform.TransformVectorNoScale(LinearVelocity);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS