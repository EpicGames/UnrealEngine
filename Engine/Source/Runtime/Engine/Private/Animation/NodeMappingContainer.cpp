// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/NodeMappingContainer.h"
#include "Engine/Blueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NodeMappingContainer)

////////////////////////////////////////////////////////////////////////////////////////
UNodeMappingContainer::UNodeMappingContainer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
void UNodeMappingContainer::AddMapping(const FName& InSourceNode, const FName& InTargetNode)
{
	if (SourceItems.Find(InSourceNode) && (InTargetNode == NAME_None || TargetItems.Find(InTargetNode)))
	{
		FName& TargetNode = SourceToTarget.Add(InSourceNode);
		TargetNode = InTargetNode;
	}
}

void UNodeMappingContainer::DeleteMapping(const FName& InSourceNode)
	{
	SourceToTarget.Remove(InSourceNode);
	}

UObject* UNodeMappingContainer::GetSourceAsset()
{
	if (!SourceAsset.IsValid())
	{
		SourceAsset.LoadSynchronous();
}

	return SourceAsset.Get();
}

UObject* UNodeMappingContainer::GetTargetAsset()
{
	if (!TargetAsset.IsValid())
	{
		TargetAsset.LoadSynchronous();
	}

	return TargetAsset.Get();
}

FString UNodeMappingContainer::GetDisplayName() const
{
	return SourceAsset.GetAssetName();
}

void UNodeMappingContainer::SetAsset(UObject* InAsset, TMap<FName, FNodeItem>& OutItems)
{
	if (InAsset)
	{
		const UBlueprint* BPAsset = Cast<UBlueprint>(InAsset);
		INodeMappingProviderInterface* Interface = nullptr;

		// if BP Asset, finding interface goes to CDO
		if (BPAsset)
{
			UObject* BPAssetCDO = BPAsset->GeneratedClass->GetDefaultObject();
			Interface = Cast<INodeMappingProviderInterface>(BPAssetCDO);
		}
		else
	{
			Interface = Cast<INodeMappingProviderInterface>(InAsset);
	}

		// once we find interface
		if (Interface)
		{
			TArray<FName> Names;
			TArray<FNodeItem> NodeItems;

			// get node items
			Interface->GetMappableNodeData(Names, NodeItems);

			// ensure they both matches
			if (ensure(Names.Num() == NodeItems.Num()))
			{
				for (int32 Index = 0; Index < Names.Num(); ++Index)
				{
					FNodeItem& ItemValue = OutItems.Add(Names[Index]);
					ItemValue = NodeItems[Index];
				}
			}
		}
	}
}

void UNodeMappingContainer::RefreshDataFromAssets()
{
	SetSourceAsset(GetSourceAsset());
	SetTargetAsset(GetTargetAsset());
}

void UNodeMappingContainer::SetSourceAsset(UObject* InSourceAsset)
{
	// we just set this all the time since the source asset may have changed or not
	SourceAsset = InSourceAsset;
	SetAsset(InSourceAsset, SourceItems);

	// verify if the mapping is still valid. 
	// delete that doesn't exists
	ValidateMapping();
}

void UNodeMappingContainer::SetTargetAsset(UObject* InTargetAsset)
{
	// we just set this all the time since the source asset may have changed or not
	TargetAsset = InTargetAsset;
	SetAsset(InTargetAsset, TargetItems);

	// verify if the mapping is still valid. 
	// delete that doesn't exists
	ValidateMapping();
}

void UNodeMappingContainer::ValidateMapping()
{
	TArray<FName> ItemsToRemove;

	for (auto Iter = SourceToTarget.CreateIterator(); Iter; ++Iter)
	{
		// make sure both exists still
		if (!SourceItems.Find(Iter.Key()) || !TargetItems.Find(Iter.Value()))
	{
			ItemsToRemove.Add(Iter.Key());
		}
	}

	// remove the list
	for (int32 Index = 0; Index < ItemsToRemove.Num(); ++Index)
	{
		SourceToTarget.Remove(ItemsToRemove[Index]);
	}
}

void UNodeMappingContainer::AddDefaultMapping()
{
	// this is slow - editor only functionality
	for (auto Iter = SourceItems.CreateConstIterator(); Iter; ++Iter)
	{
		const FName& SourceName = Iter.Key();

		// see if target has it
		if (TargetItems.Contains(SourceName))
		{
			// if so,  add to mapping
			AddMapping(SourceName, SourceName);
		}
		}
	}

#endif // WITH_EDITOR

void UNodeMappingContainer::GetTargetToSourceMappingTable(TMap<FName, FName>& OutMappingTable) const
{
	OutMappingTable.Reset();
	for (auto Iter = SourceToTarget.CreateConstIterator(); Iter; ++Iter)
	{
		// this will have issue if it has same value for multiple sources
		FName& Value = OutMappingTable.FindOrAdd(Iter.Value());
		Value = Iter.Key();
	}
}

