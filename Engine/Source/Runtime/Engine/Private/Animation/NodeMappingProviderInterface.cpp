// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/NodeMappingProviderInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NodeMappingProviderInterface)

UNodeMappingProviderInterface::UNodeMappingProviderInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

