// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/PoseAsset.h"

#include "BonePose.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "AnimationRuntime.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectAnnotation.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseAsset)

#define LOCTEXT_NAMESPACE "PoseAsset"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FPoseDataContainer
//
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////


#if WITH_EDITOR
namespace UE { namespace Anim { FUObjectAnnotationSparseBool GRegeneratedPoseAssets; } }
#endif // WITH_EDITOR

bool FPoseDataContainer::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
#endif

	return false;
}

void FPoseDataContainer::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if(Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimationRemoveSmartNames)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for(const FSmartName& PoseName : PoseNames_DEPRECATED)
		{
			PoseFNames.Add(PoseName.DisplayName);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

	RebuildCurveIndexTable();
}

void FPoseDataContainer::Reset()
{
	// clear everything
	PoseFNames.Reset();
	SortedCurveIndices.Reset();
	Poses.Reset();
	Tracks.Reset();
	TrackBoneIndices.Reset();
	Curves.Reset();
}

void FPoseDataContainer::GetPoseCurve(const FPoseData* PoseData, FBlendedCurve& OutCurve) const
{
	if (PoseData)
	{
		const TArray<float>& CurveValues = PoseData->CurveData;
		checkSlow(CurveValues.Num() == Curves.Num());

		auto GetNameFromIndex = [this](int32 InCurveIndex)
		{
			return Curves[SortedCurveIndices[InCurveIndex]].GetName();
		};

		auto GetValueFromIndex = [this, &CurveValues](int32 InCurveIndex)
		{
			return CurveValues[SortedCurveIndices[InCurveIndex]];
		};

		UE::Anim::FCurveUtils::BuildSorted(OutCurve, CurveValues.Num(), GetNameFromIndex, GetValueFromIndex, OutCurve.GetFilter());
	}
}

void FPoseDataContainer::BlendPoseCurve(const FPoseData* PoseData, FBlendedCurve& InOutCurve, float Weight) const
{
	if (PoseData)
	{
		const TArray<float>& CurveValues = PoseData->CurveData;
		checkSlow(CurveValues.Num() == Curves.Num());

		FBlendedCurve Curve;
		Curve.SetFilter(InOutCurve.GetFilter());

		GetPoseCurve(PoseData, Curve);

		UE::Anim::FNamedValueArrayUtils::Union(InOutCurve, Curve,
			[Weight](UE::Anim::FCurveElement& InOutElement, const UE::Anim::FCurveElement& InElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				InOutElement.Value = InElement.Value * Weight + InOutElement.Value;
				InOutElement.Flags |= InElement.Flags;
			});
	}
}

void FPoseDataContainer::BlendPoseCurve(const FPoseData* PoseData, FPoseAssetExtractionContext* InOutContext, float Weight) const
{
	if (PoseData)
	{
		const TArray<float>& CurveValues = PoseData->CurveData;
		checkSlow(CurveValues.Num() == Curves.Num());

		FBlendedCurve Curve;
		Curve.SetFilter(InOutContext->GetCurveFilter());

		GetPoseCurve(PoseData, Curve);

		InOutContext->BlendPoseCurve(Curve, Weight);
	}
}

FPoseData* FPoseDataContainer::FindPoseData(FName PoseName)
{
	int32 PoseIndex = PoseFNames.Find(PoseName);
	if (PoseIndex != INDEX_NONE)
	{
		return &Poses[PoseIndex];
	}

	return nullptr;
}

FPoseData* FPoseDataContainer::FindOrAddPoseData(FName PoseName)
{
	int32 PoseIndex = PoseFNames.Find(PoseName);
	if (PoseIndex == INDEX_NONE)
	{
		PoseIndex = PoseFNames.Add(PoseName);
		verify(PoseIndex == Poses.AddZeroed(1));
	}

	return &Poses[PoseIndex];
}

FTransform FPoseDataContainer::GetDefaultTransform(const FName& InTrackName, USkeleton* InSkeleton, const TArray<FTransform>& RefPose) const
{
	if (InSkeleton)
	{
		int32 SkeletonIndex = InSkeleton->GetReferenceSkeleton().FindBoneIndex(InTrackName);
		return GetDefaultTransform(SkeletonIndex, RefPose);
	}

	return FTransform::Identity;
}

FTransform FPoseDataContainer::GetDefaultTransform(int32 SkeletonIndex, const TArray<FTransform>& RefPose) const
{
	if (RefPose.IsValidIndex(SkeletonIndex))
	{
		return RefPose[SkeletonIndex];
	}

	return FTransform::Identity;
}


#if WITH_EDITOR
void FPoseDataContainer::AddOrUpdatePose(const FName& InPoseName, const TArray<FTransform>& InLocalSpacePose, const TArray<float>& InCurveData)
{
	// make sure the transforms and curves are the correct size
	if (ensureAlways(InLocalSpacePose.Num() == Tracks.Num()) && ensureAlways(InCurveData.Num() == Curves.Num()))
	{
		// find or add pose data
		FPoseData* PoseDataPtr = FindOrAddPoseData(InPoseName);
		// now add pose
		PoseDataPtr->SourceLocalSpacePose = InLocalSpacePose;
		PoseDataPtr->SourceCurveData = InCurveData;
	}

	// for now we only supports same tracks
}

bool FPoseDataContainer::InsertTrack(const FName& InTrackName, USkeleton* InSkeleton, const TArray<FTransform>& RefPose)
{
	check(InSkeleton);

	// make sure the transform is correct size
	if (Tracks.Contains(InTrackName) == false)
	{
		int32 SkeletonIndex = InSkeleton->GetReferenceSkeleton().FindBoneIndex(InTrackName);
		int32 TrackIndex = INDEX_NONE;
		if (SkeletonIndex != INDEX_NONE)
		{
			Tracks.Add(InTrackName);
			TrackIndex = Tracks.Num() - 1;

			// now insert default refpose
			const FTransform DefaultPose = GetDefaultTransform(SkeletonIndex, RefPose);

			for (FPoseData& PoseData : Poses)
			{
				ensureAlways(PoseData.SourceLocalSpacePose.Num() == TrackIndex);

				PoseData.SourceLocalSpacePose.Add(DefaultPose);

				// make sure they always match
				ensureAlways(PoseData.SourceLocalSpacePose.Num() == Tracks.Num());
			}

			return true;
		}

		return false;
	}

	return false;
}

bool FPoseDataContainer::FillUpSkeletonPose(FPoseData* PoseData, const USkeleton* InSkeleton)
{
	if (PoseData)
	{
		int32 TrackIndex = 0;
		const TArray<FTransform>& RefPose = InSkeleton->GetRefLocalPoses();
		for (const int32& SkeletonIndex : TrackBoneIndices)
		{
			PoseData->SourceLocalSpacePose[TrackIndex] = RefPose[SkeletonIndex];
			++TrackIndex;
		}

		return true;
	}

	return false;
}

void FPoseDataContainer::RenamePose(FName OldPoseName, FName NewPoseName)
{
	int32 PoseIndex = PoseFNames.Find(OldPoseName);
	if (PoseIndex != INDEX_NONE)
	{
		PoseFNames[PoseIndex] = NewPoseName;
	}
}

int32 FPoseDataContainer::DeletePose(FName PoseName)
{
	int32 PoseIndex = PoseFNames.Find(PoseName);
	if (PoseIndex != INDEX_NONE)
	{
		PoseFNames.RemoveAt(PoseIndex);
		Poses.RemoveAt(PoseIndex);
		return PoseIndex;
	}

	return INDEX_NONE;
}

bool FPoseDataContainer::DeleteCurve(FName CurveName)
{
	for (int32 CurveIndex = 0; CurveIndex < Curves.Num(); ++CurveIndex)
	{
		if (Curves[CurveIndex].GetName() == CurveName)
		{
			Curves.RemoveAt(CurveIndex);

			// delete this index from all poses
			for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
			{
				Poses[PoseIndex].CurveData.RemoveAt(CurveIndex);
				Poses[PoseIndex].SourceCurveData.RemoveAt(CurveIndex);
			}

			return true;
		}
	}

	return false;
}

void FPoseDataContainer::RetrieveSourcePoseFromExistingPose(bool bAdditive, int32 InBasePoseIndex, const TArray<FTransform>& InBasePose, const TArray<float>& InBaseCurve)
{
	for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
	{
		FPoseData& PoseData = Poses[PoseIndex];

		// if this pose is not base pose
		if (bAdditive && PoseIndex != InBasePoseIndex)
		{
			PoseData.SourceLocalSpacePose.Reset(InBasePose.Num());
			PoseData.SourceLocalSpacePose.AddUninitialized(InBasePose.Num());

			PoseData.SourceCurveData.Reset(InBaseCurve.Num());
			PoseData.SourceCurveData.AddUninitialized(InBaseCurve.Num());

			// should it be move? Why? I need that buffer still
			TArray<FTransform> AdditivePose = PoseData.LocalSpacePose;
			const ScalarRegister AdditiveWeight(1.f);

			check(AdditivePose.Num() == InBasePose.Num());
			for (int32 BoneIndex = 0; BoneIndex < AdditivePose.Num(); ++BoneIndex)
			{
				PoseData.SourceLocalSpacePose[BoneIndex] = InBasePose[BoneIndex];
				PoseData.SourceLocalSpacePose[BoneIndex].AccumulateWithAdditiveScale(AdditivePose[BoneIndex], AdditiveWeight);
			}

			int32 CurveNum = Curves.Num();
			checkSlow(CurveNum == PoseData.CurveData.Num());
			for (int32 CurveIndex = 0; CurveIndex < CurveNum; ++CurveIndex)
			{
				PoseData.SourceCurveData[CurveIndex] = InBaseCurve[CurveIndex] + PoseData.CurveData[CurveIndex];
			}
		}
		else
		{
			// otherwise, the base pose is the one
			PoseData.SourceLocalSpacePose = PoseData.LocalSpacePose;
			PoseData.SourceCurveData = PoseData.CurveData;
		}
	}
}

// this marks dirty tracks for each pose 
void FPoseDataContainer::ConvertToFullPose(USkeleton* InSkeleton, const TArray<FTransform>& RefPose)
{
	TrackPoseInfluenceIndices.Reset();
	TrackPoseInfluenceIndices.SetNum(Tracks.Num());
	
	// first create pose buffer that only has valid data
	for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
	{
		FPoseData& Pose = Poses[PoseIndex];
		check(Pose.SourceLocalSpacePose.Num() == Tracks.Num());
		Pose.LocalSpacePose.Reset();
		if (InSkeleton)
		{
			for (int32 TrackIndex = 0; TrackIndex < Tracks.Num(); ++TrackIndex)
			{
				// we only add to local space poses if it's not same as default pose
				FTransform DefaultTransform = GetDefaultTransform(Tracks[TrackIndex], InSkeleton, RefPose);
				if (!Pose.SourceLocalSpacePose[TrackIndex].Equals(DefaultTransform, UE_KINDA_SMALL_NUMBER))
				{
					int32 NewIndex = Pose.LocalSpacePose.Add(Pose.SourceLocalSpacePose[TrackIndex]);
					TrackPoseInfluenceIndices[TrackIndex].Influences.Emplace(FPoseAssetInfluence{PoseIndex, NewIndex});
				}
			}
		}

		// for now we just copy curve directly
		Pose.CurveData = Pose.SourceCurveData;
	}
}

void FPoseDataContainer::ConvertToAdditivePose(const TArray<FTransform>& InBasePose, const TArray<float>& InBaseCurve)
{
	check(InBaseCurve.Num() == Curves.Num());
	const FTransform AdditiveIdentity(FQuat::Identity, FVector::ZeroVector, FVector::ZeroVector);

	TrackPoseInfluenceIndices.Reset();
	TrackPoseInfluenceIndices.SetNum(Tracks.Num());
	
	for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
	{
		FPoseData& PoseData = Poses[PoseIndex];
		// set up buffer
		PoseData.LocalSpacePose.Reset();
		PoseData.CurveData.Reset(PoseData.SourceCurveData.Num());
		PoseData.CurveData.AddUninitialized(PoseData.SourceCurveData.Num());

		check(PoseData.SourceLocalSpacePose.Num() == InBasePose.Num());
		for (int32 BoneIndex = 0; BoneIndex < InBasePose.Num(); ++BoneIndex)
		{
			// we only add to local space poses if it has any changes in additive
			FTransform NewTransform = PoseData.SourceLocalSpacePose[BoneIndex];
			FAnimationRuntime::ConvertTransformToAdditive(NewTransform, InBasePose[BoneIndex]);
			if (!NewTransform.Equals(AdditiveIdentity))
			{
				const int32 Index = PoseData.LocalSpacePose.Add(NewTransform);
				TrackPoseInfluenceIndices[BoneIndex].Influences.Emplace(FPoseAssetInfluence{PoseIndex, Index});
			}
		}

		int32 CurveNum = Curves.Num();
		checkSlow(CurveNum == PoseData.CurveData.Num());
		for (int32 CurveIndex = 0; CurveIndex < CurveNum; ++CurveIndex)
		{
			PoseData.CurveData[CurveIndex] = PoseData.SourceCurveData[CurveIndex] - InBaseCurve[CurveIndex];
		}
	}
}
#endif // WITH_EDITOR

void FPoseDataContainer::RebuildCurveIndexTable()
{
	// Recreate sorted curve index table
	SortedCurveIndices.SetNumUninitialized(Curves.Num());
	for(int32 NameIndex = 0; NameIndex < SortedCurveIndices.Num(); ++NameIndex)
	{
		SortedCurveIndices[NameIndex] = NameIndex;
	}

	SortedCurveIndices.Sort([&Curves = Curves](int32 LHS, int32 RHS)
	{
		return Curves[LHS].GetName().FastLess(Curves[RHS].GetName());
	});
}
/////////////////////////////////////////////////////
// UPoseAsset
/////////////////////////////////////////////////////
UPoseAsset::UPoseAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAdditivePose(false)
	, BasePoseIndex(-1)
{
}

/**
 * Local utility struct that keeps skeleton bone index and compact bone index together for retargeting
 */
struct FBoneIndices
{
	int32 SkeletonBoneIndex;
	FCompactPoseBoneIndex CompactBoneIndex;

	FBoneIndices(int32 InSkeletonBoneIndex, FCompactPoseBoneIndex InCompactBoneIndex)
		: SkeletonBoneIndex(InSkeletonBoneIndex)
		, CompactBoneIndex(InCompactBoneIndex)
	{}
};

struct FPoseAssetEvalData : public TThreadSingleton<FPoseAssetEvalData>
{
	TArray<FBoneIndices> BoneIndices;
	TArray<int32> PoseWeightedIndices;
	TArray<float> PoseWeights;
	TArray<bool> WeightedPoses;
};

void UPoseAsset::GetBaseAnimationPose(struct FCompactPose& OutPose, FBlendedCurve& OutCurve) const
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData OutPoseData(OutPose, OutCurve, TempAttributes);
	GetBaseAnimationPose(OutPoseData);
}

void UPoseAsset::GetBaseAnimationPose(FAnimationPoseData& OutAnimationPoseData) const
{
	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	if (bAdditivePose && PoseContainer.Poses.IsValidIndex(BasePoseIndex))
	{
		FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();

		const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();
		USkeleton* MySkeleton = GetSkeleton();

		OutPose.ResetToRefPose();

		// this contains compact bone pose list that this pose cares
		FPoseAssetEvalData& EvalData = FPoseAssetEvalData::Get();
		TArray<FBoneIndices>& BoneIndices = EvalData.BoneIndices;
        const int32 TrackNum = PoseContainer.Tracks.Num();
		BoneIndices.SetNumUninitialized(TrackNum, EAllowShrinking::No);

		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(GetSkeleton(), RequiredBones.GetSkeletonAsset());
		for(int32 TrackIndex = 0; TrackIndex < TrackNum; ++TrackIndex)
		{
			const int32 SkeletonBoneIndex = SkeletonRemapping.IsValid() ? SkeletonRemapping.GetTargetSkeletonBoneIndex(PoseContainer.TrackBoneIndices[TrackIndex]) : PoseContainer.TrackBoneIndices[TrackIndex];
			const FCompactPoseBoneIndex PoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonIndex(SkeletonBoneIndex);
			// we add even if it's invalid because we want it to match with track index
			BoneIndices[TrackIndex].SkeletonBoneIndex = SkeletonBoneIndex;
			BoneIndices[TrackIndex].CompactBoneIndex = PoseBoneIndex;
		}

		const TArray<FTransform>& PoseTransform = PoseContainer.Poses[BasePoseIndex].LocalSpacePose;

		for (int32 TrackIndex = 0; TrackIndex < TrackNum; ++TrackIndex)
		{
			const FBoneIndices& LocalBoneIndices = BoneIndices[TrackIndex];

			if (LocalBoneIndices.CompactBoneIndex != INDEX_NONE)
			{
				FTransform& OutTransform = OutPose[LocalBoneIndices.CompactBoneIndex];
				OutTransform = PoseTransform[TrackIndex];
				FAnimationRuntime::RetargetBoneTransform(MySkeleton, GetRetargetTransformsSourceName(), GetRetargetTransforms(), OutTransform, LocalBoneIndices.SkeletonBoneIndex, LocalBoneIndices.CompactBoneIndex, RequiredBones, false);
			}
		}

		PoseContainer.GetPoseCurve(&PoseContainer.Poses[BasePoseIndex], OutCurve);
	}
	else
	{
		OutPose.ResetToRefPose();
	}
}

/*
 * The difference between BlendFromIdentityAndAccumulcate is scale
 * This ADDS scales to the FinalAtom. We use additive identity as final atom, so can't use
 */
FORCEINLINE void BlendFromIdentityAndAccumulateAdditively_Custom(FTransform& FinalAtom, const FTransform& SourceAtom, float BlendWeight)
{
	const  FTransform AdditiveIdentity(FQuat::Identity, FVector::ZeroVector, FVector::ZeroVector);

	FTransform Delta = SourceAtom;
	// Scale delta by weight
	if (BlendWeight < (1.f - ZERO_ANIMWEIGHT_THRESH))
	{
		Delta.Blend(AdditiveIdentity, Delta, BlendWeight);
	}

	FinalAtom.SetRotation(Delta.GetRotation() * FinalAtom.GetRotation());
	FinalAtom.SetTranslation(FinalAtom.GetTranslation() + Delta.GetTranslation());
	// this ADDS scale
	FinalAtom.SetScale3D(FinalAtom.GetScale3D() + Delta.GetScale3D());

	FinalAtom.DiagnosticCheckNaN_All();

	FinalAtom.NormalizeRotation();
}

void UPoseAsset::GetAnimationCurveOnly(TArray<FName>& InCurveNames, TArray<float>& InCurveValues, TArray<FName>& OutCurveNames, TArray<float>& OutCurveValues) const
{
	// if we have any pose curve
	if (ensure(InCurveNames.Num() == InCurveValues.Num()) && InCurveNames.Num() > 0)
	{
		USkeleton* MySkeleton = GetSkeleton();
		check(MySkeleton);

		FPoseAssetEvalData& EvalData = FPoseAssetEvalData::Get();
		const int32 NumPoses = PoseContainer.Poses.Num();
		TArray<float>& PoseWeights = EvalData.PoseWeights;		
		PoseWeights.Reset();
		PoseWeights.SetNumZeroed(NumPoses, EAllowShrinking::No);

		TArray<int32>& WeightedPoseIndices = EvalData.PoseWeightedIndices;	
		WeightedPoseIndices.Reset();

		bool bNormalizeWeight = bAdditivePose == false;
		float TotalWeight = 0.f;
		// we iterate through to see if we have that corresponding pose
		for (int32 CurveIndex = 0; CurveIndex < InCurveNames.Num(); ++CurveIndex)
		{
			int32 PoseIndex = PoseContainer.PoseFNames.Find(InCurveNames[CurveIndex]);
			if (ensure(PoseIndex != INDEX_NONE))
			{
				const FPoseData& PoseData = PoseContainer.Poses[PoseIndex];
				const float Value = InCurveValues[CurveIndex];

				// we only add to the list if it's not additive Or if it's additive, we don't want to add base pose index
				// and has weight
				if ((!bAdditivePose || PoseIndex != BasePoseIndex) && FAnimationRuntime::HasWeight(Value))
				{
					TotalWeight += Value;												
					PoseWeights[PoseIndex] = Value;
					WeightedPoseIndices.Add(PoseIndex);
				}
			}
		}

		const int32 TotalNumberOfValidPoses = WeightedPoseIndices.Num();
		if (TotalNumberOfValidPoses > 0)
		{
			// blend curves
			FBlendedCurve BlendedCurve;

			//if full pose, we'll have to normalize by weight
			if (bNormalizeWeight && TotalWeight > 1.f)
			{
				for (const int32& WeightedPoseIndex : WeightedPoseIndices)
				{
					float& PoseWeight = PoseWeights[WeightedPoseIndex];
					PoseWeight /= TotalWeight;

					const FPoseData& Pose = PoseContainer.Poses[WeightedPoseIndex];
					PoseContainer.BlendPoseCurve(&Pose, BlendedCurve, PoseWeight);
				}
			}
			else
			{
				for (const int32& WeightedPoseIndex : WeightedPoseIndices)
				{				
					const FPoseData& Pose = PoseContainer.Poses[WeightedPoseIndex];
					const float& PoseWeight = PoseWeights[WeightedPoseIndex];

					PoseContainer.BlendPoseCurve(&Pose, BlendedCurve, PoseWeight);
				}
			}

			OutCurveNames.Reset();
			OutCurveValues.Reset();

			BlendedCurve.ForEachElement([&OutCurveNames, &OutCurveValues](const UE::Anim::FCurveElement& InElement)
			{
				OutCurveNames.Add(InElement.Name);
				OutCurveValues.Add(InElement.Value);
			});
		}
	}
}

bool UPoseAsset::GetAnimationPose(struct FCompactPose& OutPose, FBlendedCurve& OutCurve, const FAnimExtractContext& ExtractionContext) const
{
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData OutPoseData(OutPose, OutCurve, TempAttributes);
	return GetAnimationPose(OutPoseData, ExtractionContext);
}

class FPoseAssetAnimBPExtractionContext : public FPoseAssetExtractionContext
{
public:

	FPoseAssetAnimBPExtractionContext(const FAnimExtractContext& InExtractionContext, struct FAnimationPoseData& OutAnimationPoseData)
		: FPoseAssetExtractionContext()
		, ExtractionContext(InExtractionContext)
		, AnimationPoseData(OutAnimationPoseData)
		, RetargetBoneTransformContext(OutAnimationPoseData.GetPose().GetBoneContainer())
	{
	}

	virtual int32 NumPoses() const override
	{
		return ExtractionContext.PoseCurves.Num();
	}
	
	virtual const FName& GetPoseName(const int32& InPoseIndex) const override
	{
		return ExtractionContext.PoseCurves[InPoseIndex].Name;
	}

	virtual int32 GetPoseIndexInPoseAsset(const int32& InPoseIndex) const override
	{
		return ExtractionContext.PoseCurves[InPoseIndex].PoseIndex;
	}

	virtual float GetPoseWeight(const int32& InPoseIndex) const override
	{
		return ExtractionContext.PoseCurves[InPoseIndex].Value;
	}

	virtual const USkeleton* GetTargetSkeleton(bool bEvenIfUnreachable) const override
	{
		return AnimationPoseData.GetPose().GetBoneContainer().GetSkeletonAsset(bEvenIfUnreachable);
	}

	virtual bool IsOutputBoneRequired(const int32& InOutputBoneIndex) const override
	{
		return ExtractionContext.IsBoneRequired(InOutputBoneIndex);
	}

	virtual int32 GetOutputPoseBoneIndexFromSkeletonPoseBoneIndex(const int32& InSkeletonBoneIndex) const override
	{
		return AnimationPoseData.GetPose().GetBoneContainer().GetCompactPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(InSkeletonBoneIndex)).GetInt();
	}

	virtual const UE::Anim::FCurveFilter* GetCurveFilter() const override
	{
		return AnimationPoseData.GetCurve().GetFilter();
	}

	virtual void ResetPoseToAdditiveIdentity() override
	{
		AnimationPoseData.GetPose().ResetToAdditiveIdentity();
	}

	virtual void ResetPoseToRefPose() override
	{
		AnimationPoseData.GetPose().ResetToRefPose();
	}

	virtual FTransform& GetOutputTransform(const int32& InOutputBoneIndex) override
	{
		return AnimationPoseData.GetPose()[FCompactPoseBoneIndex(InOutputBoneIndex)];
	}

	virtual const FAnimationRuntimeRetargetBoneTransformContext* GetRetargetBoneTransformContext() const override
	{
		return &RetargetBoneTransformContext;
	}

	virtual int32 NumCurves() const override
	{
		return AnimationPoseData.GetCurve().Num();
	}

	virtual void BlendPoseCurve(const FBlendedCurve& InCurve, float InWeight) override
	{
		UE::Anim::FNamedValueArrayUtils::Union(AnimationPoseData.GetCurve(), InCurve,
			[InWeight](UE::Anim::FCurveElement& InOutElement, const UE::Anim::FCurveElement& InElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				InOutElement.Value = InElement.Value * InWeight + InOutElement.Value;
				InOutElement.Flags |= InElement.Flags;
			});
	}

private:

	const FAnimExtractContext& ExtractionContext;
	struct FAnimationPoseData& AnimationPoseData;
	FBoneContainerRetargetBoneTransformContext RetargetBoneTransformContext;
};

bool UPoseAsset::GetAnimationPose(struct FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext, const bool bIgnoreSourceAnimationCurves, const bool bAllowNegativeWeights) const
{
	FPoseAssetAnimBPExtractionContext PoseAssetExtractionContext(ExtractionContext, OutAnimationPoseData);
	return GetAnimationPose(&PoseAssetExtractionContext, bIgnoreSourceAnimationCurves, bAllowNegativeWeights);
}

bool UPoseAsset::GetAnimationPose(FPoseAssetExtractionContext* InOutContext, const bool bIgnoreSourceAnimationCurves, const bool bAllowNegativeWeights) const
{
	ANIM_MT_SCOPE_CYCLE_COUNTER(PoseAssetGetAnimationPose, !IsInGameThread());
	check(InOutContext);

	// if we have any pose curve
	if (InOutContext->NumPoses())
	{
		USkeleton* MySkeleton = GetSkeleton();
		
		check(PoseContainer.IsValid());

		if (bAdditivePose)
		{
			InOutContext->ResetPoseToAdditiveIdentity();
		}
		else
		{
			InOutContext->ResetPoseToRefPose();
		}

		const int32 TrackNum = PoseContainer.Tracks.Num();

		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(GetSkeleton(), InOutContext->GetTargetSkeleton());
		// Single pose optimized evaluation path - explicitly used by PoseByName Animation Node
		if (InOutContext->NumPoses() == 1)
		{
			FPoseCurve Curve = InOutContext->GetPoseCurve(0);
			const int32 PoseIndex = Curve.PoseIndex; 
			if (ensure(PoseIndex != INDEX_NONE))
			{
				const FPoseData& Pose = PoseContainer.Poses[PoseIndex];
				// Clamp weight for non-additive pose assets rather than normalizing the weight
				const float Weight = bAdditivePose ? Curve.Value : FMath::Clamp(Curve.Value, 0.f, 1.f);
				const float WeightToCheck = bAllowNegativeWeights ? FMath::Abs(Weight) : Weight;

				// Only generate pose if the single weight is actually relevant
				if(FAnimWeight::IsRelevant(WeightToCheck))
				{
					if (!bIgnoreSourceAnimationCurves)
					{
						// Blend curve
						PoseContainer.BlendPoseCurve(&Pose, InOutContext, Weight);
					}

					// Per-track (bone) transform
					for (int32 TrackIndex = 0; TrackIndex < TrackNum; ++TrackIndex)
					{
						const FSkeletonPoseBoneIndex SkeletonBoneIndex = FSkeletonPoseBoneIndex(SkeletonRemapping.IsValid() ? SkeletonRemapping.GetTargetSkeletonBoneIndex(PoseContainer.TrackBoneIndices[TrackIndex]) : PoseContainer.TrackBoneIndices[TrackIndex]);
						const FCompactPoseBoneIndex CompactIndex(InOutContext->GetOutputPoseBoneIndexFromSkeletonPoseBoneIndex(SkeletonBoneIndex.GetInt()));

						// If bone index is invalid, or not required for the pose - skip
						if (!CompactIndex.IsValid() || !InOutContext->IsOutputBoneRequired(CompactIndex.GetInt()))
						{
							continue;
						}
					
						// Check if this track is part of the pose
						const TArray<FPoseAssetInfluence>& PoseInfluences = PoseContainer.TrackPoseInfluenceIndices[TrackIndex].Influences;
						const int32 InfluenceIndex = PoseInfluences.IndexOfByPredicate([PoseIndex](const FPoseAssetInfluence& Influence) -> bool
						{
							return Influence.PoseIndex == PoseIndex;
						});

						if (InfluenceIndex != INDEX_NONE)
						{
							FTransform& OutBoneTransform = InOutContext->GetOutputTransform(CompactIndex.GetInt());
							const FPoseAssetInfluence& Influence = PoseInfluences[InfluenceIndex];
							const int32& BonePoseIndex = Influence.BoneTransformIndex;

							const FPoseData& PoseData = PoseContainer.Poses[PoseIndex];
							const FTransform& BonePose = PoseData.LocalSpacePose[BonePoseIndex];

							// Apply additive, overriede or blend using pose weight
							if (bAdditivePose)
							{
								BlendFromIdentityAndAccumulateAdditively_Custom(OutBoneTransform, BonePose, Weight);
							}
							else if (FAnimWeight::IsFullWeight(Weight))
							{
								OutBoneTransform = BonePose;
							}
							else
							{
								OutBoneTransform = OutBoneTransform * ScalarRegister( 1 - Weight);
								OutBoneTransform.AccumulateWithShortestRotation(BonePose, ScalarRegister(Weight));
							}

							// Retarget the bone transform
							FAnimationRuntime::RetargetBoneTransform(MySkeleton, GetRetargetTransformsSourceName(), GetRetargetTransforms(), OutBoneTransform, SkeletonBoneIndex.GetInt(), CompactIndex, InOutContext->GetRetargetBoneTransformContext(), bAdditivePose);
							OutBoneTransform.NormalizeRotation();
						}
					}					
					
					return true;
				}
			}
		}
		else
		{
			// TLS storage for working data
			FPoseAssetEvalData& EvalData = FPoseAssetEvalData::Get();

			// this contains compact bone pose list that this pose cares
			TArray<FBoneIndices>& BoneIndices = EvalData.BoneIndices;
			BoneIndices.SetNumUninitialized(TrackNum, EAllowShrinking::No);

			for(int32 TrackIndex = 0; TrackIndex < TrackNum; ++TrackIndex)
			{
				const FSkeletonPoseBoneIndex SkeletonBoneIndex = FSkeletonPoseBoneIndex(SkeletonRemapping.IsValid() ? SkeletonRemapping.GetTargetSkeletonBoneIndex(PoseContainer.TrackBoneIndices[TrackIndex]) : PoseContainer.TrackBoneIndices[TrackIndex]);
				const FCompactPoseBoneIndex CompactIndex(InOutContext->GetOutputPoseBoneIndexFromSkeletonPoseBoneIndex(SkeletonBoneIndex.GetInt()));

				// we add even if it's invalid because we want it to match with track index
				BoneIndices[TrackIndex].SkeletonBoneIndex = SkeletonBoneIndex.GetInt();
				BoneIndices[TrackIndex].CompactBoneIndex = CompactIndex;
			}

			// you could only have morphtargets
			// so can't return here yet when bone indices is empty
			const bool bNormalizeWeight = bAdditivePose == false;
			const int32 NumPoses = PoseContainer.Poses.Num();
			TArray<float>& PoseWeights = EvalData.PoseWeights;
			PoseWeights.Reset();
			PoseWeights.SetNumZeroed(NumPoses, EAllowShrinking::No);

			TArray<int32>& WeightedPoseIndices = EvalData.PoseWeightedIndices;
			WeightedPoseIndices.Reset();

			TArray<bool>& WeightedPoses = EvalData.WeightedPoses;
			WeightedPoses.Reset();
			WeightedPoses.SetNumZeroed(NumPoses, EAllowShrinking::No);

			float TotalWeight = 0.f;
			// we iterate through to see if we have that corresponding pose

			const int32 NumPoseCurves = InOutContext->NumPoses();
			for (int32 CurveIndex = 0; CurveIndex < NumPoseCurves; ++CurveIndex)
			{
				const FPoseCurve Curve = InOutContext->GetPoseCurve(CurveIndex);
				const int32 PoseIndex = Curve.PoseIndex; 
				if (ensure(PoseIndex != INDEX_NONE))
				{
					const FPoseData& PoseData = PoseContainer.Poses[PoseIndex];
					const float Value = Curve.Value;

					// we only add to the list if it's not additive Or if it's additive, we don't want to add base pose index
					// and has weight
					const float WeightToCheck = bAllowNegativeWeights ? FMath::Abs(Value) : Value;
					if ((!bAdditivePose || PoseIndex != BasePoseIndex) && FAnimationRuntime::HasWeight(WeightToCheck))
					{
						TotalWeight += WeightToCheck;

						// Set pose weight and bit, and add weighted pose index
						PoseWeights[PoseIndex] = Value;
						WeightedPoseIndices.Add(PoseIndex);
						WeightedPoses[PoseIndex] = true;
					}
				}
			}

			const int32 TotalNumberOfValidPoses = WeightedPoseIndices.Num();
			if (TotalNumberOfValidPoses > 0)
			{
				//if full pose, we'll have to normalize by weight
				if (bNormalizeWeight && TotalWeight > 1.f)
				{
					for (const int32& WeightedPoseIndex : WeightedPoseIndices)
					{
						float& PoseWeight = PoseWeights[WeightedPoseIndex];
						PoseWeight /= TotalWeight;

						if (!bIgnoreSourceAnimationCurves)
						{
							// Do curve blending inline as we are looping over weights anyway
							const FPoseData& Pose = PoseContainer.Poses[WeightedPoseIndex];
							PoseContainer.BlendPoseCurve(&Pose, InOutContext, PoseWeight);
						}
					}
				}
				else
				{
					if (!bIgnoreSourceAnimationCurves)
					{
						// Take the matching curve weights from the selected poses, and blend them using the
						// weighting that we need from each pose. This is much faster than grabbing each
						// blend curve and blending them in their entirety, especially when there are very
						// few active curves for each pose and many curves for the entire pose asset.
						for (const int32& WeightedPoseIndex : WeightedPoseIndices)
						{
							const FPoseData& Pose = PoseContainer.Poses[WeightedPoseIndex];
							const float& Weight = PoseWeights[WeightedPoseIndex];
							PoseContainer.BlendPoseCurve(&Pose, InOutContext, Weight);
						}
					}
				}

				// Final per-track (bone) transform
				FTransform OutBoneTransform;
				for (int32 TrackIndex = 0; TrackIndex < TrackNum; ++TrackIndex)
				{
					const FCompactPoseBoneIndex& CompactIndex = BoneIndices[TrackIndex].CompactBoneIndex;

					// If bone index is invalid, or not required for the pose - skip
					if (CompactIndex == INDEX_NONE || !InOutContext->IsOutputBoneRequired(CompactIndex.GetInt()))
					{
						continue;
					}
					
					const TArray<FPoseAssetInfluence>& PoseInfluences = PoseContainer.TrackPoseInfluenceIndices[TrackIndex].Influences;

					// When additive, or for any bone that has no pose influences. Set transform to input.
					if (bAdditivePose || PoseInfluences.Num() == 0)
					{
						OutBoneTransform = InOutContext->GetOutputTransform(CompactIndex.GetInt());
					}

					const int32 NumInfluences = PoseInfluences.Num();
					if (NumInfluences)
					{
						float TotalLocalWeight = 0.f;
						bool bSet = false;
						// Only loop over poses known to influence this track its final transform
						for (int32 Index = 0; Index < NumInfluences; ++Index)
						{
							const FPoseAssetInfluence& Influence = PoseInfluences[Index];
							const int32& PoseIndex = Influence.PoseIndex;
							const int32& BonePoseIndex = Influence.BoneTransformIndex;
							
							// Only processs pose if its weighted
							if(WeightedPoses[PoseIndex])
							{
								const float& Weight = PoseWeights[PoseIndex];
								TotalLocalWeight += FMath::Abs(Weight);
							
								const FPoseData& PoseData = PoseContainer.Poses[PoseIndex];
								const FTransform& BonePose = PoseData.LocalSpacePose[BonePoseIndex];

								// Set weighted value For first pose, if applicable
								if(!bSet && !bAdditivePose)
								{
									OutBoneTransform = BonePose * ScalarRegister(Weight);
									bSet = true;
								}
								else
								{
									if (bAdditivePose)
									{
										BlendFromIdentityAndAccumulateAdditively_Custom(OutBoneTransform, BonePose, Weight);
									}
									else
									{
										OutBoneTransform.AccumulateWithShortestRotation(BonePose, ScalarRegister(Weight));
									}
								}
							}
						}

						// In case no influencing poses had any weight, set transform to input
						if(!FAnimWeight::IsRelevant(TotalLocalWeight))
						{
							OutBoneTransform = InOutContext->GetOutputTransform(CompactIndex.GetInt());
						}
						else if (!FAnimWeight::IsFullWeight(TotalLocalWeight) && !bAdditivePose)
						{
							OutBoneTransform.AccumulateWithShortestRotation(InOutContext->GetOutputTransform(CompactIndex.GetInt()), ScalarRegister(1.f - TotalLocalWeight));
						}
					}

					// Retarget the blended transform, and copy to output pose
					FAnimationRuntime::RetargetBoneTransform(MySkeleton, GetRetargetTransformsSourceName(), GetRetargetTransforms(), OutBoneTransform,  BoneIndices[TrackIndex].SkeletonBoneIndex, CompactIndex, InOutContext->GetRetargetBoneTransformContext(), bAdditivePose);

					InOutContext->GetOutputTransform(CompactIndex.GetInt()) = OutBoneTransform;
					InOutContext->GetOutputTransform(CompactIndex.GetInt()).NormalizeRotation();
				}

				return true;
			}
		}		
	}

	return false;
}

bool UPoseAsset::IsPostLoadThreadSafe() const
{
	return WITH_EDITORONLY_DATA == 0;	// Not thread safe in editor as the skeleton can be modified on version upgrade
}

void UPoseAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// moved to PostLoad because Skeleton is not completely loaded when we do this in Serialize
	// and we need Skeleton
	if (GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::PoseAssetSupportPerBoneMask && GetLinkerCustomVersion(FAnimPhysObjectVersion::GUID) >= FAnimPhysObjectVersion::SaveEditorOnlyFullPoseForPoseAsset)
	{
		// fix curve names
		// copy to source local data FIRST
		for (FPoseData& Pose : PoseContainer.Poses)
		{
			Pose.SourceCurveData = Pose.CurveData;
			Pose.SourceLocalSpacePose = Pose.LocalSpacePose;
		}

		PostProcessData();
	}

	if (GetLinkerCustomVersion(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::SaveEditorOnlyFullPoseForPoseAsset)
	{
		TArray<FTransform>	BasePose;
		TArray<float>		BaseCurves;
		// since the code change, the LocalSpacePose will have to be copied here manually
		// RemoveUnnecessaryTracksFromPose removes LocalSpacePose data, so we're not using it for getting base pose
		if (PoseContainer.Poses.IsValidIndex(BasePoseIndex))
		{
			BasePose = PoseContainer.Poses[BasePoseIndex].LocalSpacePose;
			BaseCurves = PoseContainer.Poses[BasePoseIndex].CurveData;
			check(BasePose.Num() == PoseContainer.Tracks.Num());
		}
		else
		{
			GetBasePoseTransform(BasePose, BaseCurves);
		}

		PoseContainer.RetrieveSourcePoseFromExistingPose(bAdditivePose, GetBasePoseIndex(), BasePose, BaseCurves);
	}

	if (GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::PoseAssetSupportPerBoneMask &&
		GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::RemoveUnnecessaryTracksFromPose)
	{
		// fix curve names
		PostProcessData();
	}

  	if(GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::PoseAssetRuntimeRefactor)
    {
		PostProcessData();
    }

	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::PoseAssetRawDataGUIDUpdate)
	{
		if (SourceAnimation)
		{
			// Fully load the source animation to ensure its RawDataGUID is populated
			SourceAnimation->ConditionalPreload();
			SourceAnimation->ConditionalPostLoad();

			const FGuid DataModelGuid = GetSourceAnimationGuid();
			if (SourceAnimationRawDataGUID.IsValid() && SourceAnimationRawDataGUID != DataModelGuid)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("AssetName"), FText::FromString(GetPathName()));
				Args.Add(TEXT("SourceAsset"), FText::FromString(SourceAnimation->GetPathName()));

				Args.Add(TEXT("Stored"), FText::FromString(SourceAnimationRawDataGUID.ToString()));
				Args.Add(TEXT("Found"), FText::FromString(DataModelGuid.ToString()));
				
				const FText ResultText = FText::Format(LOCTEXT("PoseAssetSourceOutOfDate", "PoseAsset {AssetName} is out-of-date with its source animation {SourceAsset} {Stored} vs {Found}"), Args);
				UE_LOGF(LogAnimation, Warning,"%ls", *ResultText.ToString());
			}
		}	
	}	

	// fix curve names
	USkeleton* MySkeleton = GetSkeleton();
	if (MySkeleton)
	{
		// double loop but this check only should happen once per asset
		// this should continue to add if skeleton hasn't been saved either 
		if (GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::MoveCurveTypesToSkeleton 
			|| MySkeleton->GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::MoveCurveTypesToSkeleton)
		{
			// fix up curve flags to skeleton
			for (FAnimCurveBase& Curve : PoseContainer.Curves)
			{
				bool bMorphtargetSet = Curve.GetCurveTypeFlag(AACF_DriveMorphTarget_DEPRECATED);
				bool bMaterialSet = Curve.GetCurveTypeFlag(AACF_DriveMaterial_DEPRECATED);

				// only add this if that has to 
				if (bMorphtargetSet || bMaterialSet)
				{
					MySkeleton->AccumulateCurveMetaData(Curve.GetName(), bMaterialSet, bMorphtargetSet);
				}
			}
		}
	}

	// I have to fix pose names
	if(RemoveInvalidTracks())
	{
		PostProcessData();
	}
	else
#endif // WITH_EDITORONLY_DATA
	{
		UpdateTrackBoneIndices();
	}
}

void UPoseAsset::Serialize(FArchive& Ar)
{
 	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	Super::Serialize(Ar);
}

void UPoseAsset::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	if (!ObjectSaveContext.IsProceduralSave())
	{
		UpdateRetargetSourceAssetData();
	}
#endif // WITH_EDITOR
	Super::PreSave(ObjectSaveContext);
}

void UPoseAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	// Number of poses
	Context.AddTag(FAssetRegistryTag("Poses", FString::FromInt(GetNumPoses()), FAssetRegistryTag::TT_Numerical));
#if WITH_EDITOR
	TArray<FName> Names;
	Names.Reserve(PoseContainer.PoseFNames.Num() + PoseContainer.Curves.Num());

	for (const FName& PoseName : PoseContainer.PoseFNames)
	{
		Names.Add(PoseName);
	}

	for (const FAnimCurveBase& Curve : PoseContainer.Curves)
	{
		Names.AddUnique(Curve.GetName());
	}
	
	// Add curve IDs to a tag list, or a delimiter if we have no curves.
	// The delimiter is necessary so we can distinguish between data with no curves and old data, as the asset registry
	// strips tags that have empty values 
	FString PoseNameList = USkeleton::CurveTagDelimiter;
	for(const FName& Name : Names)
	{
		PoseNameList += FString::Printf(TEXT("%s%s"), *Name.ToString(), *USkeleton::CurveTagDelimiter);
	}
	Context.AddTag(FAssetRegistryTag(USkeleton::CurveNameTag, PoseNameList, FAssetRegistryTag::TT_Hidden)); //write pose names as curve tag as they use 
#endif
}

int32 UPoseAsset::GetNumPoses() const
{ 
	return PoseContainer.GetNumPoses();
}

int32 UPoseAsset::GetNumCurves() const
{
	return PoseContainer.Curves.Num();
}

int32 UPoseAsset::GetNumTracks() const
{
	return PoseContainer.Tracks.Num();
}

const TArray<FName>& UPoseAsset::GetPoseFNames() const
{
	return PoseContainer.PoseFNames;
}

const TArray<FName>& UPoseAsset::GetTrackNames() const
{
	return PoseContainer.Tracks;
}

const TArray<FName> UPoseAsset::GetCurveFNames() const
{
	TArray<FName> CurveNames;
	for (int32 CurveIndex = 0; CurveIndex < PoseContainer.Curves.Num(); ++CurveIndex)
	{
		CurveNames.Add(PoseContainer.Curves[CurveIndex].GetName());
	}

	return CurveNames;
}

const TArray<FAnimCurveBase>& UPoseAsset::GetCurveData() const
{
	return PoseContainer.Curves;
}

const TArray<float>& UPoseAsset::GetCurveValues(const int32 PoseIndex) const
{
	return GetRawCurveValues(PoseIndex);
}

#if WITH_EDITOR

const TArray<float>& UPoseAsset::GetFullCurveValues(const int32 PoseIndex) const
{
	if (PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		return PoseContainer.Poses[PoseIndex].SourceCurveData;
	}

	static const TArray<float> EmptyCurveValues;
	return EmptyCurveValues;
}

#endif

const TArray<float>& UPoseAsset::GetRawCurveValues(const int32 PoseIndex) const
{
	if (PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		return PoseContainer.Poses[PoseIndex].CurveData;
	}

	static const TArray<float> EmptyCurveValues;
	return EmptyCurveValues;
}

bool UPoseAsset::GetCurveValue(const int32 PoseIndex, const int32 CurveIndex, float& OutValue) const
{
	bool bSuccess = false;

	if (PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		const FPoseData& PoseData = PoseContainer.Poses[PoseIndex];
		if (PoseData.CurveData.IsValidIndex(CurveIndex))
		{
			OutValue = PoseData.CurveData[CurveIndex];
			bSuccess = true;
		}
	}

	return bSuccess;
}

const int32 UPoseAsset::GetTrackIndexByName(const FName& InTrackName) const
{
	int32 ResultTrackIndex = INDEX_NONE;

	// Only search if valid name passed in
	if (InTrackName != NAME_None)
	{
		ResultTrackIndex = PoseContainer.Tracks.Find(InTrackName);
	}

	return ResultTrackIndex;
}


bool UPoseAsset::ContainsPose(const FName& InPoseName) const
{
	for (const FName& PoseName : PoseContainer.PoseFNames)
	{
		if (PoseName == InPoseName)
		{
			return true;
		}
	}

	return false;
}

#if WITH_EDITOR
void UPoseAsset::RenamePose(const FName& OriginalPoseName, const FName& NewPoseName)
{
	ModifyPoseName(OriginalPoseName, NewPoseName);
}

FName UPoseAsset::GetBasePoseName() const
{
	return PoseContainer.PoseFNames.IsValidIndex(BasePoseIndex) ? PoseContainer.PoseFNames[BasePoseIndex] : NAME_None;
}

bool UPoseAsset::SetBasePoseName(const FName& NewBasePoseName)
{
	if (NewBasePoseName != NAME_None)
	{
		const int32 NewIndex = PoseContainer.PoseFNames.IndexOfByKey(NewBasePoseName);
		if (NewIndex != INDEX_NONE)
		{
			BasePoseIndex = NewIndex;
			return true;
		}
		else
		{
			return false;
		}
	}
	
	BasePoseIndex = INDEX_NONE;
	return true;
}

void UPoseAsset::GetPoseNames(TArray<FName>& PoseNames) const
{	
	const int32 NumPoses = GetNumPoses();
	for (int32 PoseIndex = 0; PoseIndex < NumPoses; ++PoseIndex)
	{
		PoseNames.Add(GetPoseNameByIndex(PoseIndex));
	}
}
// whenever you change SourceLocalPoses, or SourceCurves, we should call this to update runtime dataa
void UPoseAsset::PostProcessData()
{
	RemoveInvalidTracks();
	
	// convert back to additive if it was that way
	if (bAdditivePose)
	{
		ConvertToAdditivePose(GetBasePoseIndex());
	}
	else
	{
		ConvertToFullPose();
	}

	UpdateTrackBoneIndices();

	PoseContainer.RebuildCurveIndexTable();
}

void UPoseAsset::BreakAnimationSequenceGUIDComparison()
{
	SourceAnimationRawDataGUID.Invalidate();
}

FName UPoseAsset::AddPoseWithUniqueName(const USkeletalMeshComponent* MeshComponent)
{
	const FName NewPoseName = GetUniquePoseName(this);
	AddOrUpdatePose(NewPoseName, MeshComponent);

	PostProcessData();

	OnPoseListChanged.Broadcast();

	return NewPoseName;
}

void UPoseAsset::AddReferencePose(const FName& PoseName, const FReferenceSkeleton& RefSkeleton)
{
	TArray<FTransform> BoneTransforms;
	TArray<FName> TrackNames;

	const TArray<FTransform>& ReferencePose = RefSkeleton.GetRefBonePose();
	for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
	{
		TrackNames.Add(RefSkeleton.GetBoneName(BoneIndex));
		BoneTransforms.Add(ReferencePose[BoneIndex]);
	}
			
	TArray<float> NewCurveValues;
	NewCurveValues.AddZeroed(PoseContainer.Curves.Num());

	AddOrUpdatePose(PoseName, TrackNames, BoneTransforms, NewCurveValues);
	PostProcessData();
}

void UPoseAsset::AddOrUpdatePose(const FName& PoseName, const USkeletalMeshComponent* MeshComponent, bool bUpdateCurves)
{
	const USkeleton* MySkeleton = GetSkeleton();
	if (MySkeleton && MeshComponent && MeshComponent->GetSkeletalMeshAsset())
	{
		TArray<FName> TrackNames;
		// note this ignores root motion
		TArray<FTransform> BoneTransform = MeshComponent->GetComponentSpaceTransforms();
		const FReferenceSkeleton& RefSkeleton = MeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton();
		for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
		{
			TrackNames.Add(RefSkeleton.GetBoneName(BoneIndex));
		}

		// convert to local space
		for (int32 BoneIndex = BoneTransform.Num() - 1; BoneIndex >= 0; --BoneIndex)
		{
			const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				BoneTransform[BoneIndex] = BoneTransform[BoneIndex].GetRelativeTransform(BoneTransform[ParentIndex]);
			}
		}
		
		TArray<float> NewCurveValues;
		NewCurveValues.AddZeroed(PoseContainer.Curves.Num());

		const FBlendedHeapCurve& MeshCurves = MeshComponent->GetAnimationCurves();

		for (int32 NewCurveIndex = 0; NewCurveIndex < NewCurveValues.Num(); ++NewCurveIndex)
		{
			const FAnimCurveBase& Curve = PoseContainer.Curves[NewCurveIndex];
			const float MeshCurveValue = MeshCurves.Get(Curve.GetName());
			NewCurveValues[NewCurveIndex] = MeshCurveValue;
		}
		
		BreakAnimationSequenceGUIDComparison();

		// Only update curves if user has requested so - or when setting up a new pose
		const FPoseData* PoseData = PoseContainer.FindPoseData(PoseName);
		AddOrUpdatePose(PoseName, TrackNames, BoneTransform, (PoseData && !bUpdateCurves) ? PoseData->CurveData : NewCurveValues);
		PostProcessData();
	}
}

void UPoseAsset::AddOrUpdatePose(const FName& PoseName, const TArray<FName>& TrackNames, const TArray<FTransform>& LocalTransform, const TArray<float>& CurveValues)
{
	const USkeleton* MySkeleton = GetSkeleton();
	if (MySkeleton)
	{
		// first combine track, we want to make sure all poses contains tracks with this
		CombineTracks(TrackNames);

		const bool bNewPose = PoseContainer.FindPoseData(PoseName) == nullptr;
		FPoseData* PoseData = PoseContainer.FindOrAddPoseData(PoseName);
		// now copy all transform back to it. 
		check(PoseData);
		// Make sure this is whole tracks, not tracknames
		// TrackNames are what this pose contains
		// but We have to add all tracks to match poses container
		// TrackNames.Num() is subset of PoseContainer.Tracks.Num()
		// Above CombineTracks will combine both
		const int32 TotalTracks = PoseContainer.Tracks.Num();
		PoseData->SourceLocalSpacePose.Reset(TotalTracks);
		PoseData->SourceLocalSpacePose.AddUninitialized(TotalTracks);
		PoseData->SourceLocalSpacePose.SetNumZeroed(TotalTracks, EAllowShrinking::Yes);

		// just fill up skeleton pose
		// the reason we use skeleton pose, is that retarget source can change, and 
		// it can miss the tracks. 
		PoseContainer.FillUpSkeletonPose(PoseData, MySkeleton);
		check(CurveValues.Num() == PoseContainer.Curves.Num());
		PoseData->SourceCurveData = CurveValues;

		// why do we need skeleton index
		//const FReferenceSkeleton& RefSkeleton = MySkeleton->GetReferenceSkeleton();
		for (int32 Index = 0; Index < TrackNames.Num(); ++Index)
		{
			// now get poseData track index
			const FName& TrackName = TrackNames[Index];
			//int32 SkeletonIndex = RefSkeleton.FindBoneIndex(TrackName);
			const int32 InternalTrackIndex = PoseContainer.Tracks.Find(TrackName);
			// copy to the internal track index
			PoseData->SourceLocalSpacePose[InternalTrackIndex] = LocalTransform[Index];
		}
				
		BreakAnimationSequenceGUIDComparison();

		if (bNewPose)
		{
			OnPoseListChanged.Broadcast();
		}
	}
}
void UPoseAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		bool bConvertToAdditivePose = false;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseAsset, RetargetSourceAsset))
		{
			bConvertToAdditivePose = true;
			UpdateRetargetSourceAssetData();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseAsset, RetargetSource))
		{
			bConvertToAdditivePose = true;
		}

		if (bConvertToAdditivePose)
		{
			USkeleton* MySkeleton = GetSkeleton();
			if (MySkeleton)
			{
				// Convert to additive again since retarget source changed
				ConvertToAdditivePose(GetBasePoseIndex());
			}
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseAsset, SourceAnimation))
		{
			BreakAnimationSequenceGUIDComparison();
		}
	}
}

void UPoseAsset::CombineTracks(const TArray<FName>& NewTracks)
{
	USkeleton* MySkeleton = GetSkeleton();
	if (MySkeleton)
	{
		for (const FName& NewTrack : NewTracks)
		{
			if (PoseContainer.Tracks.Contains(NewTrack) == false)
			{
				// if we don't have it, then we'll have to add this track and then 
				// right now it doesn't have to be in the hierarchy
				// @todo: it is probably best to keep the hierarchy of the skeleton, so in the future, we might like to sort this by track after
				PoseContainer.InsertTrack(NewTrack, MySkeleton, GetRetargetTransforms());
				UpdateTrackBoneIndices();
			}
		}
	}
}

void UPoseAsset::Reinitialize()
{
	PoseContainer.Reset();

	bAdditivePose = false;
	BasePoseIndex = INDEX_NONE;
}

void UPoseAsset::RenamePoseOrCurveName(const FName& InOriginalName, const FName& InNewName)
{
	if(PoseContainer.PoseFNames.Contains(InNewName) || PoseContainer.Curves.ContainsByPredicate([InNewName](const FAnimCurveBase& InCurve){ return InCurve.GetName() == InNewName; }))
	{
		// Cant rename on top of something we already have - this will create duplicates
		return;
	}

	for (FName Name : PoseContainer.PoseFNames)
	{
		if (Name == InOriginalName)
		{
			Name = InNewName;
			break;
		}
	}

	for (FAnimCurveBase& Curve : PoseContainer.Curves)
	{
		if (Curve.GetName() == InOriginalName)
		{
			Curve.SetName(InNewName);
			break;
		}
	}

	PoseContainer.RebuildCurveIndexTable();
}

void UPoseAsset::RemovePoseOrCurveNames(const TArray<FName>& InNamesToRemove)
{
	DeletePoses(InNamesToRemove);
	DeleteCurves(InNamesToRemove);
}

void UPoseAsset::CreatePoseFromAnimation(class UAnimSequence* AnimSequence, const TArray<FName>* InPoseNames/*== nullptr*/)
{
	if (AnimSequence)
	{
		USkeleton* TargetSkeleton = AnimSequence->GetSkeleton();
		if (TargetSkeleton)
		{
			SetSkeleton(TargetSkeleton);
			SourceAnimation = AnimSequence;

			// reinitialize, now we're making new pose from this animation
			Reinitialize();

			int32 NumPoses = AnimSequence->GetNumberOfSampledKeys();
			if(InPoseNames && InPoseNames->Num() > NumPoses)
			{
				NumPoses=InPoseNames->Num();
			}
			// make sure we have more than one pose
			if (NumPoses > 0)
			{
				// stack allocator for extracting curve
				FMemMark Mark(FMemStack::Get());

				// set up track data - @todo: add revaliation code when checked
				IAnimationDataModel* DataModel = AnimSequence->GetDataModel();

				TArray<FName> TrackNames;
				DataModel->GetBoneTrackNames(TrackNames);

				for (const FName& TrackName : TrackNames)
				{
					PoseContainer.Tracks.Add(TrackName);
				}

				// now create pose transform
				TArray<FTransform> NewPose;
				
				const int32 NumTracks = TrackNames.Num();
				NewPose.Reset(NumTracks);
				NewPose.AddUninitialized(NumTracks);
				
				const double IntervalBetweenKeys = (NumPoses > 1)? AnimSequence->GetPlayLength() / (NumPoses -1 ) : 0.f;

				// add curves - only float curves
				const FAnimationCurveData& AnimationCurveData = DataModel->GetCurveData();
				const int32 TotalFloatCurveCount = AnimationCurveData.FloatCurves.Num();

				if (TotalFloatCurveCount > 0)
				{
					for (const FFloatCurve& Curve : AnimationCurveData.FloatCurves)
					{
						PoseContainer.Curves.Add(FAnimCurveBase(Curve.GetName(), Curve.GetCurveTypeFlags()));
					}
				}

				// add to skeleton UID, so that it knows the curve data
				for (int32 PoseIndex = 0; PoseIndex < NumPoses; ++PoseIndex)
				{
					TArray<float> CurveData;
					CurveData.Reserve(TotalFloatCurveCount);
					
					FName NewPoseName = (InPoseNames && InPoseNames->IsValidIndex(PoseIndex))? (*InPoseNames)[PoseIndex] : GetUniquePoseName(this);
					// now get rawanimationdata, and each key is converted to new pose
					for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
					{
						NewPose[TrackIndex] = AnimSequence->GetDataModel()->GetBoneTrackTransform(TrackNames[TrackIndex], FFrameNumber(PoseIndex));
					}

					if (TotalFloatCurveCount > 0)
					{
						// get curve data
						// have to do iterate over time
						// support curve
						FBlendedCurve SourceCurve;
						AnimSequence->EvaluateCurveData(SourceCurve, FAnimExtractContext(PoseIndex*IntervalBetweenKeys), true);
						
						SourceCurve.ForEachElement([&CurveData](const UE::Anim::FCurveElement& InElement)
						{
							CurveData.Add(InElement.Value);
						});
						check(CurveData.Num() == PoseContainer.Curves.Num());
					}
				
					// add new pose
					PoseContainer.AddOrUpdatePose(NewPoseName, NewPose, CurveData);
				}

				PostProcessData();
			}

			// Mark PoseAsset as (re-)generated
			UE::Anim::GRegeneratedPoseAssets.Set(this);
			
			SourceAnimationRawDataGUID = GetSourceAnimationGuid();
		}
	}
}

void UPoseAsset::UpdatePoseFromAnimation(class UAnimSequence* AnimSequence)
{
	if (AnimSequence)
	{
		// when you update pose, right now, it just only keeps pose names
		// in the future we might want to make it more flexible
		// back up old pose names
		const TArray<FName> OldPoseNames = PoseContainer.PoseFNames;
		const bool bOldAdditive = bAdditivePose;
		int32 OldBasePoseIndex = BasePoseIndex;
		CreatePoseFromAnimation(AnimSequence, &OldPoseNames);

		// fix up additive info if it's additive
		if (bOldAdditive)
		{
			if (PoseContainer.Poses.IsValidIndex(OldBasePoseIndex) == false)
			{
				// if it's pointing at invalid index, just reset to ref pose
				OldBasePoseIndex = INDEX_NONE;
			}

			// Convert to additive again
			ConvertToAdditivePose(OldBasePoseIndex);
		}

		PoseContainer.RebuildCurveIndexTable();

		OnPoseListChanged.Broadcast();
	}
}

bool UPoseAsset::ModifyPoseName(FName OldPoseName, FName NewPoseName)
{
	if (ContainsPose(NewPoseName))
	{
		// already exists, return 
		return false;
	}

	if (FPoseData* PoseData = PoseContainer.FindPoseData(OldPoseName))
	{
		PoseContainer.RenamePose(OldPoseName, NewPoseName);
		OnPoseListChanged.Broadcast();

		return true;
	}

	return false;
}

int32 UPoseAsset::DeletePoses(TArray<FName> PoseNamesToDelete)
{
	int32 ItemsDeleted = 0;

	for (const FName& PoseName : PoseNamesToDelete)
	{
		int32 PoseIndexDeleted = PoseContainer.DeletePose(PoseName);
		if (PoseIndexDeleted != INDEX_NONE)
		{
			++ItemsDeleted;
			// if base pose index is same as pose index deleted
			if (BasePoseIndex == PoseIndexDeleted)
			{
				BasePoseIndex = INDEX_NONE;
			}
			// if base pose index is after this, we reduce the number
			else if (BasePoseIndex > PoseIndexDeleted)
			{
				--BasePoseIndex;
			}
		}
	}
	
	if (ItemsDeleted)
	{
		BreakAnimationSequenceGUIDComparison();
	}

	PostProcessData();
	OnPoseListChanged.Broadcast();

	return ItemsDeleted;
}

int32 UPoseAsset::DeleteCurves(TArray<FName> CurveNamesToDelete)
{
	int32 ItemsDeleted = 0;

	for (const FName& CurveName : CurveNamesToDelete)
	{
		if(PoseContainer.DeleteCurve(CurveName))
		{
			++ItemsDeleted;
		}
	}

	OnPoseListChanged.Broadcast();

	return ItemsDeleted;
}

void UPoseAsset::ConvertToFullPose()
{
	PoseContainer.ConvertToFullPose(GetSkeleton(), GetRetargetTransforms());
	bAdditivePose = false;
}

void UPoseAsset::ConvertToAdditivePose(int32 NewBasePoseIndex)
{
	// make sure it's valid
	check(NewBasePoseIndex == -1 || PoseContainer.Poses.IsValidIndex(NewBasePoseIndex));

	BasePoseIndex = NewBasePoseIndex;

	TArray<FTransform> BasePose;
	TArray<float>		BaseCurves;
	GetBasePoseTransform(BasePose, BaseCurves);

	PoseContainer.ConvertToAdditivePose(BasePose, BaseCurves);

	bAdditivePose = true;
}

bool UPoseAsset::GetFullPose(int32 PoseIndex, TArray<FTransform>& OutTransforms) const
{
	if (!PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		return false;
	}

	// just return source data
	OutTransforms = PoseContainer.Poses[PoseIndex].SourceLocalSpacePose;
	return true;
}

const TArray<FTransform>& UPoseAsset::GetFullPose(int32 PoseIndex) const
{
	if (PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		return PoseContainer.Poses[PoseIndex].SourceLocalSpacePose;
	}

	static const TArray<FTransform> EmptyTransforms;
	return EmptyTransforms;
}

#endif

bool UPoseAsset::GetRawPose(int32 PoseIndex, TArray<FTransform>& OutTransforms) const
{
	if (!PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		return false;
	}

	// just return source data
	OutTransforms = PoseContainer.Poses[PoseIndex].LocalSpacePose;
	return true;
}

const TArray<FTransform>& UPoseAsset::GetRawPose(int32 PoseIndex) const
{
	if (PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		return PoseContainer.Poses[PoseIndex].LocalSpacePose;
	}

	static const TArray<FTransform> EmptyTransforms;
	return EmptyTransforms;
}

#if WITH_EDITOR

FTransform UPoseAsset::GetComponentSpaceTransform(FName BoneName, const TArray<FTransform>& LocalTransforms) const
{
	const FReferenceSkeleton& RefSkel = GetSkeleton()->GetReferenceSkeleton();

	// Init component space transform with identity
	FTransform ComponentSpaceTransform = FTransform::Identity;

	// Start to walk up parent chain until we reach root (ParentIndex == INDEX_NONE)
	int32 BoneIndex = RefSkel.FindBoneIndex(BoneName);
	while (BoneIndex != INDEX_NONE)
	{
		BoneName = RefSkel.GetBoneName(BoneIndex);
		int32 TrackIndex = GetTrackIndexByName(BoneName);

		// If a track for parent, get local space transform from that
		// If not, get from ref pose
		FTransform BoneLocalTM = (TrackIndex != INDEX_NONE) ? LocalTransforms[TrackIndex] : RefSkel.GetRefBonePose()[BoneIndex];

		// Continue to build component space transform
		ComponentSpaceTransform = ComponentSpaceTransform * BoneLocalTM;

		// Now move up to parent
		BoneIndex = RefSkel.GetParentIndex(BoneIndex);
	}

	return ComponentSpaceTransform;
}

const FTransform& UPoseAsset::GetLocalSpaceTransform(FName BoneName, int32 PoseIndex) const
{
	const int32 TrackIdx = GetTrackIndexByName(BoneName);
	if (!PoseContainer.Poses.IsValidIndex(PoseIndex) || TrackIdx == INDEX_NONE)
	{
		UE_LOGF(LogAnimation, Error, "Can't find bone with name %ls when trying to get local space transform", *BoneName.ToString());
		return FTransform::Identity;
	}
	return PoseContainer.Poses[PoseIndex].SourceLocalSpacePose[TrackIdx];
}

const FTransform& UPoseAsset::GetLocalSpaceTransform(int32 TrackIndex, int32 PoseIndex) const
{
	if (!PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		return FTransform::Identity;
	}
	if (!PoseContainer.Poses[PoseIndex].SourceLocalSpacePose.IsValidIndex(TrackIndex))
	{
		return FTransform::Identity;
	}
	return PoseContainer.Poses[PoseIndex].SourceLocalSpacePose[TrackIndex];
}

bool UPoseAsset::ConvertSpace(bool bNewAdditivePose, int32 NewBasePoseIndex)
{
	// first convert to full pose first
	bAdditivePose = bNewAdditivePose;
	BasePoseIndex = NewBasePoseIndex;
	PostProcessData();

	return true;
}
#endif // WITH_EDITOR

const int32 UPoseAsset::GetPoseIndexByName(const FName& InBasePoseName) const
{
	for (int32 PoseIndex = 0; PoseIndex < PoseContainer.PoseFNames.Num(); ++PoseIndex)
	{
		if (PoseContainer.PoseFNames[PoseIndex] == InBasePoseName)
		{
			return PoseIndex;
		}
	}

	return INDEX_NONE;
}

const int32 UPoseAsset::GetCurveIndexByName(const FName& InCurveName) const
{
	for (int32 TestIdx = 0; TestIdx < PoseContainer.Curves.Num(); TestIdx++)
	{
		const FAnimCurveBase& Curve = PoseContainer.Curves[TestIdx];
		if (Curve.GetName() == InCurveName)
		{
			return TestIdx;
		}
	}
	return INDEX_NONE;
}


void UPoseAsset::UpdateTrackBoneIndices()
{
	USkeleton* MySkeleton = GetSkeleton();
	PoseContainer.TrackBoneIndices.Reset();
	if (MySkeleton)
	{
		const FReferenceSkeleton& RefSkeleton = MySkeleton->GetReferenceSkeleton();

		PoseContainer.TrackBoneIndices.SetNumZeroed(PoseContainer.Tracks.Num());
		for (int32 TrackIndex = 0; TrackIndex < PoseContainer.Tracks.Num(); ++TrackIndex)
		{
			const FName& TrackName = PoseContainer.Tracks[TrackIndex];
			PoseContainer.TrackBoneIndices[TrackIndex] = RefSkeleton.FindBoneIndex(TrackName);
		}
	}
}

bool UPoseAsset::RemoveInvalidTracks()
{
	const USkeleton* MySkeleton = GetSkeleton();
	const int32 InitialNumTracks = PoseContainer.Tracks.Num();

	if (MySkeleton)
	{
		const FReferenceSkeleton& RefSkeleton = MySkeleton->GetReferenceSkeleton();

		// set up track data 
		for (int32 TrackIndex = 0; TrackIndex < PoseContainer.Tracks.Num(); ++TrackIndex)
		{
			const FName& TrackName = PoseContainer.Tracks[TrackIndex];
			const int32 SkeletonTrackIndex = RefSkeleton.FindBoneIndex(TrackName);
			if (SkeletonTrackIndex == INDEX_NONE)
			{
				// delete this track. It's missing now
				PoseContainer.DeleteTrack(TrackIndex);
				--TrackIndex;
			}
		}
	}

	return InitialNumTracks != PoseContainer.Tracks.Num();
}

#if WITH_EDITOR
void UPoseAsset::SetRetargetSourceAsset(USkeletalMesh* InRetargetSourceAsset)
{
	if (InRetargetSourceAsset != nullptr && InRetargetSourceAsset->HasAnyFlags(RF_Transient))
	{
		UE_LOGF(LogAnimation, Error, "Error, Transient asset [%ls] can not be assigned as Retarget Source for Pose Asset [%ls]. Please, use a non transient asset as retarget surce."
			, *(InRetargetSourceAsset->GetFullName())
			, *GetFullName());
		ensure(false);
		return;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RetargetSourceAsset = InRetargetSourceAsset;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UPoseAsset::ClearRetargetSourceAsset()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RetargetSourceAsset.Reset();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const TSoftObjectPtr<USkeletalMesh>& UPoseAsset::GetRetargetSourceAsset() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RetargetSourceAsset;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UPoseAsset::UpdateRetargetSourceAssetData()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	USkeletalMesh* SourceReferenceMesh = RetargetSourceAsset.LoadSynchronous();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	const USkeleton* MySkeleton = GetSkeleton();
	if (SourceReferenceMesh && MySkeleton)
	{
		FAnimationRuntime::MakeSkeletonRefPoseFromMesh(SourceReferenceMesh, MySkeleton, RetargetSourceAssetReferencePose);
	}
	else
	{
		RetargetSourceAssetReferencePose.Empty();
	}
}
#endif // WITH_EDITORONLY_DATA

const TArray<FTransform>& UPoseAsset::GetRetargetTransforms() const
{
	if (RetargetSource.IsNone() && RetargetSourceAssetReferencePose.Num() > 0)
	{
		return RetargetSourceAssetReferencePose;
	}
	else
	{
		const USkeleton* MySkeleton = GetSkeleton();
		if (MySkeleton)
		{
			return MySkeleton->GetRefLocalPoses(RetargetSource);
		}
		else
		{
			static TArray<FTransform> EmptyTransformArray;
			return EmptyTransformArray;
		}
	}
}

FName UPoseAsset::GetRetargetTransformsSourceName() const
{
	if (RetargetSource.IsNone() && RetargetSourceAssetReferencePose.Num() > 0)
	{
		return GetOutermost()->GetFName();
	}
	else
	{
		return RetargetSource;
	}
}

void FPoseDataContainer::DeleteTrack(int32 TrackIndex)
{
	Tracks.RemoveAt(TrackIndex);
	for (FPoseData& Pose : Poses)
	{
#if WITH_EDITOR
		// if not editor, they can't save this data, so it will run again when editor runs
		Pose.SourceLocalSpacePose.RemoveAt(TrackIndex);
#endif // WITH_EDITOR
	}
}

#if WITH_EDITOR
FName UPoseAsset::GetUniquePoseName(UPoseAsset* PoseAsset)
{
	check(PoseAsset);
	int32 NameIndex = 0;
	FName NewName;

	do
	{
		NewName = FName(*FString::Printf(TEXT("Pose_%d"), NameIndex++));
	}
	while(PoseAsset->ContainsPose(NewName));
	
	return NewName;
}

FGuid UPoseAsset::GetSourceAnimationGuid() const
{	
	if (SourceAnimation)
	{
		IAnimationDataModel::FGuidGenerationSettings Settings;
		if (GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::AnimModelGuidGenerationSettings)
		{
			Settings.bIncludeTimingData = false;
		}		

		// If pose asset was re-generated during editor runtime, use the new GUID format
		if(UE::Anim::GRegeneratedPoseAssets.Get(this))
		{
			Settings.bIncludeTimingData = true;	
		}
		
		return SourceAnimation->GetDataModel()->GenerateGuid(Settings);
	}

	return FGuid();
}

void UPoseAsset::RemapTracksToNewSkeleton(USkeleton* NewSkeleton, bool bConvertSpaces)
{
	Super::RemapTracksToNewSkeleton(NewSkeleton, bConvertSpaces);

	PostProcessData();
}

bool UPoseAsset::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive /*= true*/)
{
	Super::GetAllAnimationSequencesReferred(AnimationAssets, bRecursive);
	if (SourceAnimation)
	{
		SourceAnimation->HandleAnimReferenceCollection(AnimationAssets, bRecursive);
	}

	return AnimationAssets.Num() > 0;
}

void UPoseAsset::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	Super::ReplaceReferredAnimations(ReplacementMap);
	if (SourceAnimation)
	{
		UAnimSequence* const* ReplacementAsset = (UAnimSequence*const*)ReplacementMap.Find(SourceAnimation);
		if (ReplacementAsset)
		{
			SourceAnimation = *ReplacementAsset;
		}
	}
}

#endif

bool UPoseAsset::GetBasePoseTransform(TArray<FTransform>& OutBasePose, TArray<float>& OutBaseCurve) const
{
	int32 TotalNumTrack = PoseContainer.Tracks.Num();
	OutBasePose.Reset(TotalNumTrack);

	if (BasePoseIndex == -1)
	{
		OutBasePose.AddUninitialized(TotalNumTrack);

		USkeleton* MySkeleton = GetSkeleton();
		if (MySkeleton)
		{
			for (int32 TrackIndex = 0; TrackIndex < PoseContainer.Tracks.Num(); ++TrackIndex)
			{
				const FName& TrackName = PoseContainer.Tracks[TrackIndex];
				OutBasePose[TrackIndex] = PoseContainer.GetDefaultTransform(TrackName, MySkeleton, GetRetargetTransforms());
			}
		}
		else
		{
			for (int32 TrackIndex = 0; TrackIndex < PoseContainer.Tracks.Num(); ++TrackIndex)
			{
				OutBasePose[TrackIndex].SetIdentity();
			}
		}

		// add zero curves
		OutBaseCurve.AddZeroed(PoseContainer.Curves.Num());
		check(OutBasePose.Num() == TotalNumTrack);
		return true;
	}
	else if (PoseContainer.Poses.IsValidIndex(BasePoseIndex))
	{
#if WITH_EDITOR
		OutBasePose = PoseContainer.Poses[BasePoseIndex].SourceLocalSpacePose;
		OutBaseCurve = PoseContainer.Poses[BasePoseIndex].SourceCurveData;
#else
		TArray<FPoseAssetInfluenceForPose> Influences;
		ConvertToFullPose(GetBasePoseIndex(), OutBasePose, Influences, OutBaseCurve);
#endif
		check(OutBasePose.Num() == TotalNumTrack);
		return true;
	}

	return false;
}

bool UPoseAsset::GetTrackInfluencesForPose(const int32 PoseIndex, TArray<FPoseAssetInfluenceForPose>& OutInfluences) const
{
	OutInfluences.Reset();

	// todo: is this right? we may need to do this properly
	if (PoseIndex == BasePoseIndex && bAdditivePose)
	{
		return true;
	}
	
	if (!PoseContainer.Poses.IsValidIndex(PoseIndex))
	{
		return false;
	}
	for (int32 TrackIndex = 0; TrackIndex < PoseContainer.TrackPoseInfluenceIndices.Num(); ++TrackIndex)
	{
		const FPoseAssetInfluences& Influences = PoseContainer.TrackPoseInfluenceIndices[TrackIndex];
		for (const FPoseAssetInfluence& Influence : Influences.Influences)
		{
			if (Influence.PoseIndex == PoseIndex)
			{
				const int32 TransformIndex = OutInfluences.Num();
				OutInfluences.Emplace(TrackIndex, TransformIndex);
				break;
			}
		}
	}
	return true;
}

bool UPoseAsset::GetTrackInfluencesForPoses(const TArrayView<const FPoseCurve>& InPoses, TArray<FPoseAssetInfluenceForPose>& OutInfluences) const
{
	OutInfluences.Reset();

	TArray<bool> IncludePose;
	IncludePose.AddZeroed(GetNumPoses());
	for (const FPoseCurve& Pose : InPoses)
	{
		if (Pose.PoseIndex == BasePoseIndex)
		{
			continue;
		}
		if (!PoseContainer.Poses.IsValidIndex(Pose.PoseIndex))
		{
			continue;
		}
		if (!FAnimWeight::IsRelevant(FMath::Abs(Pose.Value)))
		{
			continue;
		}
		IncludePose[Pose.PoseIndex] = true;
	}

	for (int32 TrackIndex = 0; TrackIndex < PoseContainer.TrackPoseInfluenceIndices.Num(); ++TrackIndex)
	{
		const FPoseAssetInfluences& Influences = PoseContainer.TrackPoseInfluenceIndices[TrackIndex];
		for (const FPoseAssetInfluence& Influence : Influences.Influences)
		{
			if (IncludePose.IsValidIndex(Influence.PoseIndex) && IncludePose[Influence.PoseIndex])
			{
				const int32 TransformIndex = OutInfluences.Num();
				OutInfluences.Emplace(TrackIndex, TransformIndex);
				break;
			}
		}
	}
	return true;
}

bool UPoseAsset::ConvertToFullPose(const int32 PoseIndex, TArray<FTransform>& OutFullPose, TArray<FPoseAssetInfluenceForPose>& OutInfluences, TArray<float>& OutFullCurves) const
{
	return
		ConvertToFullPose(PoseIndex, OutFullPose, OutInfluences) &&
		ConvertToFullPose(PoseIndex, OutFullCurves);
}

bool UPoseAsset::ConvertToFullPose(const int32 PoseIndex, TArray<FTransform>& OutFullPose, TArray<FPoseAssetInfluenceForPose>& OutInfluences) const
{
	if (!GetTrackInfluencesForPose(PoseIndex, OutInfluences))
	{
		return false;
	}

	if (PoseIndex == BasePoseIndex || !bAdditivePose)
	{
		OutFullPose = PoseContainer.Poses[PoseIndex].LocalSpacePose;
	}
	else if (BasePoseIndex == INDEX_NONE)
	{
		const TArray<FTransform>& AdditivePose = PoseContainer.Poses[PoseIndex].LocalSpacePose;
		const TArray<FTransform>& RetargetTransforms = GetRetargetTransforms();

		OutFullPose.Reset();
		OutFullPose.Reserve(OutInfluences.Num());

		for (const FPoseAssetInfluenceForPose& Influence : OutInfluences)
		{
			const FTransform& BaseTransform = PoseContainer.GetDefaultTransform(Influence.TrackIndex, RetargetTransforms);
			FTransform& BoneTransform = OutFullPose.Add_GetRef(BaseTransform);
			BlendFromIdentityAndAccumulateAdditively_Custom(BoneTransform, AdditivePose[Influence.BoneTransformIndex], 1);
		}
	}
	else
	{
		const TArray<FTransform>& AdditivePose = PoseContainer.Poses[PoseIndex].LocalSpacePose;
		const TArray<FTransform>& BasePose = PoseContainer.Poses[BasePoseIndex].LocalSpacePose;

		OutFullPose.Reset();
		OutFullPose.Reserve(OutInfluences.Num());

		for (const FPoseAssetInfluenceForPose& Influence : OutInfluences)
		{
			FTransform& BoneTransform = OutFullPose.Add_GetRef(BasePose[Influence.TrackIndex]);
			BlendFromIdentityAndAccumulateAdditively_Custom(BoneTransform, AdditivePose[Influence.BoneTransformIndex], 1);
		}
	}

	return true;
}

bool UPoseAsset::ConvertToFullPose(const int32 PoseIndex, TArray<float>& OutFullCurves) const
{
	if (PoseIndex == BasePoseIndex || BasePoseIndex == INDEX_NONE || !bAdditivePose)
	{
		OutFullCurves = PoseContainer.Poses[PoseIndex].CurveData;
	}
	else
	{
		const TArray<float>& BaseCurves = PoseContainer.Poses[BasePoseIndex].CurveData;
		const TArray<float>& AdditiveCurves = PoseContainer.Poses[PoseIndex].CurveData;
		check(BaseCurves.Num() == AdditiveCurves.Num());
		OutFullCurves.Reset();
		OutFullCurves.Reserve(BaseCurves.Num());

		for (int32 CurveIndex = 0; CurveIndex < BaseCurves.Num(); ++CurveIndex)
		{
			OutFullCurves.Add(BaseCurves[CurveIndex] + AdditiveCurves[CurveIndex]);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE 

