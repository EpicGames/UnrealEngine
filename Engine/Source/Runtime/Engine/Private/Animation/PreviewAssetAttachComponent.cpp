// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/PreviewAssetAttachComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PreviewAssetAttachComponent)

void FPreviewAssetAttachContainer::AddAttachedObject( UObject* AttachObject, FName AttachPointName )
{
	FPreviewAttachedObjectPair Pair;
	Pair.AttachedTo = AttachPointName;
	Pair.SetAttachedObject(AttachObject);
	AttachedObjects.Add( Pair );
}

void FPreviewAssetAttachContainer::AddUniqueAttachedObject(UObject* AttachObject, FName AttachPointName)
{
	for (const FPreviewAttachedObjectPair& AttachedObject : AttachedObjects)
	{
		if (AttachedObject.GetAttachedObject() == AttachObject && AttachedObject.AttachedTo == AttachPointName)
		{
			return;
		}
	}

	FPreviewAttachedObjectPair Pair;
	Pair.AttachedTo = AttachPointName;
	Pair.SetAttachedObject(AttachObject);
	AttachedObjects.Add(Pair);
}

void FPreviewAssetAttachContainer::RemoveAttachedObject( UObject* ObjectToRemove, FName AttachName )
{
	for( int i = AttachedObjects.Num() - 1; i >= 0; --i )
	{
		FPreviewAttachedObjectPair& Pair = AttachedObjects[i];

		if( Pair.GetAttachedObject() == ObjectToRemove && Pair.AttachedTo == AttachName )
		{
			AttachedObjects.RemoveAtSwap( i, EAllowShrinking::No);
			break;
		}
	}
}

UObject* FPreviewAssetAttachContainer::GetAttachedObjectByAttachName( FName AttachName ) const
{
	for( int i = 0; i < AttachedObjects.Num(); ++i )
	{
		if( AttachedObjects[i].AttachedTo == AttachName )
		{
			return AttachedObjects[i].GetAttachedObject();
		}
	}
	return NULL;
}

void FPreviewAssetAttachContainer::ClearAllAttachedObjects()
{
	AttachedObjects.Empty();
}

int32 FPreviewAssetAttachContainer::Num() const
{
	return AttachedObjects.Num();
}

FPreviewAttachedObjectPair& FPreviewAssetAttachContainer::operator []( int32 i )
{
	return AttachedObjects[i];
}

const FPreviewAttachedObjectPair& FPreviewAssetAttachContainer::operator [](int32 i) const
{
	return AttachedObjects[i];
}

TConstIterator FPreviewAssetAttachContainer::CreateConstIterator() const
{
	return AttachedObjects.CreateConstIterator();
}

TIterator FPreviewAssetAttachContainer::CreateIterator()
{
	return AttachedObjects.CreateIterator();
}

void FPreviewAssetAttachContainer::RemoveAtSwap( int32 Index, EAllowShrinking AllowShrinking /*=EAllowShrinking::Default */ )
{
	AttachedObjects.RemoveAtSwap( Index, AllowShrinking );
}

void FPreviewAssetAttachContainer::RemoveAtSwap( int32 Index, int32 Count, EAllowShrinking AllowShrinking /*=EAllowShrinking::Default */ )
{
	AttachedObjects.RemoveAtSwap( Index, Count, AllowShrinking );
}

void FPreviewAssetAttachContainer::SaveAttachedObjectsFromDeprecatedProperties()
{
	for (FPreviewAttachedObjectPair& Pair : AttachedObjects)
	{
		Pair.SaveAttachedObjectFromDeprecatedProperty();
	}
}

int32 FPreviewAssetAttachContainer::ValidatePreviewAttachedObjects()
{
	int32 NumBrokenAssets = 0;

	for (int32 i = AttachedObjects.Num() - 1; i >= 0; --i)
	{
		FPreviewAttachedObjectPair& PreviewAttachedObject = AttachedObjects[i];

		if (!PreviewAttachedObject.GetAttachedObject())
		{
			AttachedObjects.RemoveAtSwap(i);
			++NumBrokenAssets;
		}
	}

	return NumBrokenAssets;
}

