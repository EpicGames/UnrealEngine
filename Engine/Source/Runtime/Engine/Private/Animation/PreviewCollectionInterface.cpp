// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/PreviewCollectionInterface.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PreviewCollectionInterface)

UPreviewCollectionInterface::UPreviewCollectionInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
