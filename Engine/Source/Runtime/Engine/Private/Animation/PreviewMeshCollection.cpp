// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/PreviewMeshCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PreviewMeshCollection)

void UPreviewMeshCollection::GetPreviewSkeletalMeshes(TArray<USkeletalMesh*>& OutList, TArray<TSubclassOf<UAnimInstance>>& OutAnimBP) const
{
	OutList.Empty();
	OutAnimBP.Empty();
	
	for (int32 MeshIndex = 0; MeshIndex < SkeletalMeshes.Num(); ++MeshIndex)
	{
		const FPreviewMeshCollectionEntry& Entry = SkeletalMeshes[MeshIndex];

		// Load up our valid skeletal meshes
		if (Entry.SkeletalMesh.LoadSynchronous())
		{
			OutList.Add(Entry.SkeletalMesh.Get());
			TSubclassOf<UAnimInstance> SubClass = nullptr;
			// Load up our custom animation blueprints
			if (Entry.AnimBlueprint.LoadSynchronous())
			{
				const UAnimBlueprint* AnimBP = Entry.AnimBlueprint.Get();
				if(UClass* AnimInstanceClass = Cast<UClass>(AnimBP->GeneratedClass))
				{					
					SubClass = AnimInstanceClass;
				}
			}
			
			OutAnimBP.Add(SubClass);
		}
	}
}

