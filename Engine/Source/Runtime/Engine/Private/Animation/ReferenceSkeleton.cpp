// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceSkeleton.h"
#include "Animation/Skeleton.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "EngineLogs.h"
#include "Algo/Count.h"
#include "Engine/SkeletalMesh.h"

FReferenceSkeletonModifier::FReferenceSkeletonModifier(USkeleton* InSkeleton)
	: RefSkeleton(InSkeleton->ReferenceSkeleton),
	Skeleton(InSkeleton)
{}

FReferenceSkeletonModifier::~FReferenceSkeletonModifier()
{
	RefSkeleton.RebuildRefSkeleton(Skeleton, true);
}

void FReferenceSkeletonModifier::UpdateRefPoseTransform(const int32 BoneIndex, const FTransform& BonePose)
{
	RefSkeleton.UpdateRefPoseTransform(BoneIndex, BonePose);
}

void FReferenceSkeletonModifier::Add(const FMeshBoneInfo& BoneInfo, const FTransform& BonePose, const bool bAllowMultipleRoots)
{
	TGuardValue OperationGuard(RefSkeleton.bOnlyOneRootAllowed, !bAllowMultipleRoots);
	RefSkeleton.Add(BoneInfo, BonePose);
}

void FReferenceSkeletonModifier::Remove(const FName& BoneName, const bool bRemoveChildren)
{
	RefSkeleton.Remove(BoneName, bRemoveChildren);
}

void FReferenceSkeletonModifier::Rename(const FName& InOldName, const FName& InNewName)
{
	RefSkeleton.Rename(InOldName, InNewName);
}

int32 FReferenceSkeletonModifier::SetParent(const FName& InBoneName, const FName& InParentName, const bool bAllowMultipleRoots)
{
	TGuardValue OperationGuard(RefSkeleton.bOnlyOneRootAllowed, !bAllowMultipleRoots);
	return RefSkeleton.SetParent(InBoneName, InParentName);
}

int32 FReferenceSkeletonModifier::FindBoneIndex(const FName& BoneName) const
{
	return RefSkeleton.FindRawBoneIndex(BoneName);
}

const TArray<FMeshBoneInfo>& FReferenceSkeletonModifier::GetRefBoneInfo() const
{
	return RefSkeleton.GetRawRefBoneInfo();
}

FArchive &operator<<(FArchive& Ar, FMeshBoneInfo& F)
{
	Ar << F.Name << F.ParentIndex;

	if (Ar.IsLoading() && (Ar.UEVer() < VER_UE4_REFERENCE_SKELETON_REFACTOR))
	{
		FColor DummyColor = FColor::White;
		Ar << DummyColor;
	}

#if WITH_EDITORONLY_DATA
	if (Ar.UEVer() >= VER_UE4_STORE_BONE_EXPORT_NAMES)
	{
		if (!Ar.IsCooking() && !Ar.IsFilterEditorOnly())
		{
			Ar << F.ExportName;
		}
	}
	else
	{
		F.ExportName = F.Name.ToString();
	}
#endif

	return Ar;
}


//////////////////////////////////////////////////////////////////////////

FTransform GetComponentSpaceTransform(TArray<uint8>& ComponentSpaceFlags, TArray<FTransform>& ComponentSpaceTransforms, FReferenceSkeleton& RefSkeleton, int32 TargetIndex)
{
	FTransform& This = ComponentSpaceTransforms[TargetIndex];

	if (!ComponentSpaceFlags[TargetIndex])
	{
		const int32 ParentIndex = RefSkeleton.GetParentIndex(TargetIndex);
		This *= GetComponentSpaceTransform(ComponentSpaceFlags, ComponentSpaceTransforms, RefSkeleton, ParentIndex);
		ComponentSpaceFlags[TargetIndex] = 1;
	}
	return This;
}

void FReferenceSkeleton::Remove(const FName InBoneName, const bool bRemoveChildren)
{
	const int32 RawBoneIndex = FindRawBoneIndex(InBoneName);
	if (RawBoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogAnimation, Error, "Remove: '%ls' not found.", *InBoneName.ToString());
		return;
	}

	const FMeshBoneInfo& BoneInfo = RawRefBoneInfo[RawBoneIndex];
	const int32 RawParentIndex = BoneInfo.ParentIndex;

	// is this a root?
	if (RawParentIndex == INDEX_NONE)
	{
		// is this the only root?
		const int32 NumRoots = Algo::CountIf(RawRefBoneInfo, [](const FMeshBoneInfo& BoneInfo) { return BoneInfo.ParentIndex == INDEX_NONE; });
		if (NumRoots == 1)
		{
			UE_LOGF(LogAnimation, Error, "Remove: cannot remove root bone as it's the only one in the hierarchy.");
			return;
		}
	}

	// Make sure our arrays are in sync.
	checkSlow((RawRefBoneInfo.Num() == RawRefBonePose.Num()) && (RawRefBoneInfo.Num() == RawNameToIndexMap.Num()));

	// store children indices and sort them from greatest to lowest
	TArray<int32> Children;
	GetRawDirectChildBones(RawBoneIndex, Children);
	Children.Sort([](const int32 Index0, const int32 Index1) {return Index0 > Index1;} );

	// reindex function
	auto ReIndexBones = [this, RawBoneIndex, RawParentIndex](const bool bUpdateParent)
	{
		for (int32 NextIndex=RawBoneIndex+1; NextIndex < GetRawBoneNum(); NextIndex++)
		{
			FMeshBoneInfo& Bone = RawRefBoneInfo[NextIndex];
        
			// update parent
			if (Bone.ParentIndex > RawBoneIndex)
			{
				Bone.ParentIndex -= 1;
			}
			else if (bUpdateParent && (Bone.ParentIndex == RawBoneIndex))
			{
				Bone.ParentIndex = RawParentIndex;
			}
        
			// update cached index
			RawNameToIndexMap[Bone.Name] -= 1;
		}
	};

	if (bRemoveChildren)
	{
		// 1 - treat children first
		for (const int32 ChildIndex: Children)
		{
			Remove(RawRefBoneInfo[ChildIndex].Name, bRemoveChildren);
		}

		// 2 - reindex next bones
		ReIndexBones(false);
		
		// 3 - remove useless raw data
		RawRefBonePose.RemoveAt(RawBoneIndex, 1);
		RawRefBoneInfo.RemoveAt(RawBoneIndex, 1);
		RawNameToIndexMap.Remove(InBoneName);
		
		return;
	}

	// 1 - store transforms

	// store parent's global transform
	FTransform ParentGlobal = FTransform::Identity;
	int32 ParentIndex = RawParentIndex;
	while (ParentIndex > INDEX_NONE)
	{
		ParentGlobal = ParentGlobal * RawRefBonePose[ParentIndex];
		ParentIndex--;
	}

	// store bone's global transform
	const FTransform BoneGlobal = RawRefBonePose[RawBoneIndex] * ParentGlobal;

	// 2 - switch children transforms to new parent space
	for (const int32 ChildIndex: Children)
	{
		const FTransform ChildrenGlobal = RawRefBonePose[ChildIndex] * BoneGlobal;
		RawRefBonePose[ChildIndex] = ChildrenGlobal.GetRelativeTransform(ParentGlobal);
	}

	// 3 - reindex next bones
	ReIndexBones(true);

	// 4 - remove useless raw data
	RawRefBonePose.RemoveAt(RawBoneIndex, 1);
	RawRefBoneInfo.RemoveAt(RawBoneIndex, 1);
	RawNameToIndexMap.Remove(InBoneName);
}

void FReferenceSkeleton::Rename(const FName InBoneName, const FName InNewName)
{
	const int32 RawBoneIndex = FindRawBoneIndex(InBoneName);
	const bool bCaseSensitiveOnly = InBoneName == InNewName && !InBoneName.IsEqual(InNewName, ENameCase::CaseSensitive);
	const int32 RawNewBoneIndex = bCaseSensitiveOnly ? INDEX_NONE : FindRawBoneIndex(InNewName);
	if (RawBoneIndex == INDEX_NONE || RawNewBoneIndex != INDEX_NONE)
	{
		return;
	}

	FMeshBoneInfo& BoneInfo = RawRefBoneInfo[RawBoneIndex];
	BoneInfo.Name = InNewName;
#if WITH_EDITORONLY_DATA
	BoneInfo.ExportName = BoneInfo.Name.ToString();
#endif
	RawNameToIndexMap.Remove(InBoneName);
	RawNameToIndexMap.Add(InNewName, RawBoneIndex);
}

namespace FReferenceSkeletonLocals
{
	struct FElement
	{
		int32 RawIndex = INDEX_NONE;
		FElement* Parent = nullptr;
		TArray<FElement*> Children;
	};
	
	struct FHierarchy
	{
		FHierarchy(const FReferenceSkeleton& InRefSkeleton)
		{
			const TArray<FMeshBoneInfo>& Infos = InRefSkeleton.GetRawRefBoneInfo();
			const int32 NumBones = Infos.Num();
			if (NumBones > 0)
			{
				const TArray<FTransform>& Poses = InRefSkeleton.GetRawRefBonePose();

				Elements.Reserve(NumBones);
				Transforms.Reserve(NumBones);
				TBitArray<> TransformCached(false, NumBones);

				// build elements
				for (int32 Index = 0; Index < NumBones; ++Index)
				{
					FElement NewElement({ Index, nullptr, {} });
					Elements.Emplace(MoveTemp(NewElement));
					Transforms.Add(Poses[Index]);
				}

				// build hierarchy & global transforms
				auto GetGlobalTransform = [this, &TransformCached](const FElement& Element, auto&& GetGlobalTransformArg)
				{
					if (TransformCached[Element.RawIndex])
					{
						return Transforms[Element.RawIndex];
					}

					if (Element.Parent)
					{
						Transforms[Element.RawIndex] *= GetGlobalTransformArg(*Element.Parent, GetGlobalTransformArg);
						Transforms[Element.RawIndex].NormalizeRotation();
					}

					TransformCached[Element.RawIndex] = true;
					return Transforms[Element.RawIndex];
				};
				
				for (int32 Index = 0; Index < NumBones; ++Index)
				{
					SetParent(Index, Infos[Index].ParentIndex);
					GetGlobalTransform(Elements[Index], GetGlobalTransform);
				}
			}
		}

		// switch parent
		void SetParent(const int32 InChildIndex, const int32 InParentIndex)
		{
			check(Elements.IsValidIndex(InChildIndex));
			check(InParentIndex >= INDEX_NONE && InParentIndex < Elements.Num());

			FElement& Child = Elements[InChildIndex];
			if (Child.Parent)
			{
				Child.Parent->Children.Remove(&Child);
				Child.Parent = nullptr;
			}
			
			if (Elements.IsValidIndex(InParentIndex))
			{
				FElement& Parent = Elements[InParentIndex];
				Child.Parent = &Parent;
				Parent.Children.Add(&Child);
			}
		}

		// get the new to old index mapping to update the bone infos and poses
		void GetNewToOldIndexes(TArray<int32>& OutMapping)
		{
			OutMapping.Reset(0); OutMapping.Reserve(Elements.Num());
			
			TBitArray<> Visited(false, Elements.Num());

			auto ElementToIndex = [this, &Visited, &OutMapping](const FElement& Element, auto&& ElementToIndexArg) -> void
			{
				// make sure the parent is added first
				if (Element.Parent && !Visited[Element.Parent->RawIndex])
				{
					ElementToIndexArg(*Element.Parent, ElementToIndexArg);
				}
				
				// add the element
				if (!Visited[Element.RawIndex])
				{
					OutMapping.Add(Element.RawIndex);
					Visited[Element.RawIndex] = true;
				}

				// add it's direct children
				for (const FElement* Child: Element.Children)
				{
					if (!Visited[Child->RawIndex])
					{
						OutMapping.Add(Child->RawIndex);
						Visited[Child->RawIndex] = true;
					}
				}

				// recurse
				for (const FElement* Child: Element.Children)
				{
					ElementToIndexArg(*Child, ElementToIndexArg);
				}
			};

			// look for the first root has it must be inserted first
			// this can happen if the initial root has been re-parented to a new bone
			const int32 FirstRoot = Elements.IndexOfByPredicate([](const FElement& Element)
			{
				return Element.Parent == nullptr;
			});

			if (ensure(FirstRoot != INDEX_NONE))
			{
				ElementToIndex(Elements[FirstRoot], ElementToIndex);
			}
			
			// parse the full hierarchy
			for (const FElement& Element: Elements)
			{
				ElementToIndex(Element, ElementToIndex);
			}
		}

		TArray<FElement> Elements;
		TArray<FTransform> Transforms;
	};
}

int32 FReferenceSkeleton::SetParent(const FName InBoneName, const FName InParentName)
{	
	if (InBoneName == InParentName)
	{
		return INDEX_NONE;
	}

	const int32 BoneIndex = FindRawBoneIndex(InBoneName);
	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogAnimation, Error, "SetParent: bone '%ls' not found.", *InBoneName.ToString());
		return INDEX_NONE;
	}
	
	const int32 NewParentIndex = FindRawBoneIndex(InParentName);
	if (NewParentIndex == INDEX_NONE && InParentName != NAME_None)
	{
		UE_LOGF(LogAnimation, Error, "SetParent: parent bone '%ls' not found.", *InParentName.ToString());
		return INDEX_NONE;
	}

	if (RawRefBoneInfo[BoneIndex].ParentIndex == NewParentIndex)
	{
		return INDEX_NONE;
	}

	auto GetParentIndex = [&](const int32 InBoneIndex) -> int32
	{
		return RawRefBoneInfo.IsValidIndex(InBoneIndex) ? RawRefBoneInfo[InBoneIndex].ParentIndex : INDEX_NONE;
	};

	int32 ParentParentIndex = GetParentIndex(NewParentIndex); 
	while (ParentParentIndex > INDEX_NONE)
	{
		if (ParentParentIndex == BoneIndex)
		{ // we do not allow to reparent to one of bone's children
			UE_LOGF(LogAnimation, Error, "SetParent: cannot parent '%ls' to one of its children.", *InBoneName.ToString());
			return INDEX_NONE;
		}
		ParentParentIndex = GetParentIndex(ParentParentIndex);
	}

	using namespace FReferenceSkeletonLocals;
	
	// build temp hierarchy
	FHierarchy Hierarchy(*this);

	// switch parent
	Hierarchy.SetParent(BoneIndex, NewParentIndex);

	// update infos, poses and name to index mapping
	TArray<int32> NewToOldIndexes; Hierarchy.GetNewToOldIndexes(NewToOldIndexes);

	const TArray<FMeshBoneInfo>& Infos = GetRawRefBoneInfo();
	
	check(NewToOldIndexes.Num() == Infos.Num());

	const TArray<FElement>& Elements = Hierarchy.Elements;
	const TArray<FTransform>& Transforms = Hierarchy.Transforms;

	const int32 NumElements = Elements.Num();
	TArray<FMeshBoneInfo> NewRawRefBoneInfo; NewRawRefBoneInfo.Reserve(NumElements);
	TArray<FTransform> NewRawRefBonePose; NewRawRefBonePose.Reserve(NumElements);
	TMap<FName, int32> NewNameToIndexMap; NewNameToIndexMap.Reserve(NumElements);

	// recreate infos
	for (int32 Index = 0; Index < NumElements; ++Index)
	{
		const int32 NewIndex = NewToOldIndexes[Index];
		const FElement& Element = Elements[NewIndex];
		
		// add new info
		const int32 OldIndex = Element.RawIndex;
		FMeshBoneInfo& NewBoneInfo = NewRawRefBoneInfo.Add_GetRef(Infos[OldIndex]);
		
		// update parent
		int32 NewParentIdx = INDEX_NONE;
		if (const FElement* NewParent = Element.Parent)
		{
			NewParentIdx = NewToOldIndexes.IndexOfByPredicate([NewParent](int32 InIndex)
			{
				return NewParent->RawIndex == InIndex;
			});
		}
		NewBoneInfo.ParentIndex = NewParentIdx; 

		// update pose
		FTransform& Pose = NewRawRefBonePose.Add_GetRef(Transforms[NewIndex]);
		if (NewParentIdx != INDEX_NONE)
		{
			Pose = Transforms[NewIndex].GetRelativeTransform(Transforms[Element.Parent->RawIndex]);
			Pose.NormalizeRotation();
		}

		// update name to index map
		NewNameToIndexMap.Add(NewBoneInfo.Name, Index);
	}

	// swap data
	RawRefBonePose = MoveTemp(NewRawRefBonePose);
	RawRefBoneInfo = MoveTemp(NewRawRefBoneInfo);
	RawNameToIndexMap = MoveTemp(NewNameToIndexMap);

	return *RawNameToIndexMap.Find(InBoneName);
}

int32 FReferenceSkeleton::GetRawSourceBoneIndex(const USkeleton* Skeleton, const FName& SourceBoneName) const
{
	for (const FVirtualBone& VB : Skeleton->GetVirtualBones())
	{
		//Is our source another virtual bone
		if (VB.VirtualBoneName == SourceBoneName)
		{
			//return our source virtual bones target, it is the same transform
			//but it exists in the raw bone array
			return FindBoneIndex(VB.TargetBoneName);
		}
	}
	return FindBoneIndex(SourceBoneName);
}

void FReferenceSkeleton::RebuildRefSkeleton(const USkeleton* Skeleton, bool bRebuildNameMap)
{
	if (bRebuildNameMap)
	{
		//On loading FinalRefBone data wont exist but NameToIndexMap will and will be valid
		RebuildNameToIndexMap();
	}

	const int32 NumVirtualBones = Skeleton ? Skeleton->GetVirtualBones().Num() : 0;
	FinalRefBoneInfo = TArray<FMeshBoneInfo>(RawRefBoneInfo, NumVirtualBones); //-V781
	FinalRefBonePose = TArray<FTransform>(RawRefBonePose, NumVirtualBones);
	FinalNameToIndexMap = RawNameToIndexMap;

	RequiredVirtualBones.Reset(NumVirtualBones);
	UsedVirtualBoneData.Reset(NumVirtualBones);
	
	InvalidateEndOfBranchCache();

	if (NumVirtualBones > 0)
	{
		TArray<uint8> ComponentSpaceFlags;
		ComponentSpaceFlags.AddZeroed(RawRefBonePose.Num());
		ComponentSpaceFlags[0] = 1;

		TArray<FTransform> ComponentSpaceTransforms = TArray<FTransform>(RawRefBonePose);

		for (int32 VirtualBoneIdx = 0; VirtualBoneIdx < NumVirtualBones; ++VirtualBoneIdx)
		{
			const int32 ActualIndex = VirtualBoneIdx + RawRefBoneInfo.Num();
			const FVirtualBone& VB = Skeleton->GetVirtualBones()[VirtualBoneIdx];

			const int32 SourceIndex = GetRawSourceBoneIndex(Skeleton, VB.SourceBoneName);	//-V595
			const int32 ParentIndex = FindBoneIndex(VB.SourceBoneName);
			const int32 TargetIndex = FindBoneIndex(VB.TargetBoneName);
			if(ParentIndex != INDEX_NONE && TargetIndex != INDEX_NONE)
			{
				FinalRefBoneInfo.Add(FMeshBoneInfo(VB.VirtualBoneName, VB.VirtualBoneName.ToString(), ParentIndex));

				const FTransform TargetCS = GetComponentSpaceTransform(ComponentSpaceFlags, ComponentSpaceTransforms, *this, TargetIndex);
				const FTransform SourceCS = GetComponentSpaceTransform(ComponentSpaceFlags, ComponentSpaceTransforms, *this, SourceIndex);

				FTransform VBTransform = TargetCS.GetRelativeTransform(SourceCS);

				const int32 NewBoneIndex = FinalRefBonePose.Add(VBTransform);
				FinalNameToIndexMap.Add(VB.VirtualBoneName) = NewBoneIndex;
				RequiredVirtualBones.Add(NewBoneIndex);
				UsedVirtualBoneData.Add(FVirtualBoneRefData(NewBoneIndex, SourceIndex, TargetIndex));
			}
		}
	}

	if (Skeleton && (this == &Skeleton->GetReferenceSkeleton()))
	{
		// Full rebuild of all compatible with this and with ones we are compatible with.
		UE::Anim::FSkeletonRemappingRegistry::Get().RefreshMappings(Skeleton);

#if WITH_EDITOR
		// Notify the skeleton that we changed
		Skeleton->HandleReferenceSkeletonChange();
#endif
	}
}

void FReferenceSkeleton::RemoveDuplicateBones(const UObject* Requester, TArray<FBoneIndexType> & DuplicateBones)
{
	//Process raw bone data only
	const int32 NumBones = RawRefBoneInfo.Num();
	DuplicateBones.Empty();

	TMap<FName, int32> BoneNameCheck;
	bool bRemovedBones = false;
	for (int32 BoneIndex = NumBones - 1; BoneIndex >= 0; BoneIndex--)
	{
		const FName& BoneName = GetBoneName(BoneIndex);
		const int32* FoundBoneIndexPtr = BoneNameCheck.Find(BoneName);

		// Not a duplicate bone, track it.
		if (FoundBoneIndexPtr == NULL)
		{
			BoneNameCheck.Add(BoneName, BoneIndex);
		}
		else
		{
			const int32 DuplicateBoneIndex = *FoundBoneIndexPtr;
			DuplicateBones.Add(DuplicateBoneIndex);

			UE_LOGF(LogAnimation, Warning, "RemoveDuplicateBones: duplicate bone name (%ls) detected for (%ls)! Indices: %d and %d. Removing the latter.",
				*BoneName.ToString(), *GetNameSafe(Requester), DuplicateBoneIndex, BoneIndex);

			// Remove duplicate bone index, which was added later as a mistake.
			RawRefBonePose.RemoveAt(DuplicateBoneIndex, 1);
			RawRefBoneInfo.RemoveAt(DuplicateBoneIndex, 1);

			// Now we need to fix all the parent indices that pointed to bones after this in the array
			// These must be after this point in the array.
			for (int32 j = DuplicateBoneIndex; j < GetRawBoneNum(); j++)
			{
				if (GetParentIndex(j) >= DuplicateBoneIndex)
				{
					RawRefBoneInfo[j].ParentIndex -= 1;
				}
			}

			// Update entry in case problem bones were added multiple times.
			BoneNameCheck.Add(BoneName, BoneIndex);

			// We need to make sure that any bone that has this old bone as a parent is fixed up
			bRemovedBones = true;
		}
	}

	// If we've removed bones, we need to rebuild our name table.
	if (bRemovedBones || (RawNameToIndexMap.Num() == 0))
	{
		const USkeleton* Skeleton = Cast<USkeleton>(Requester);
		if (!Skeleton)
		{
			if (const USkeletalMesh* Mesh = Cast<USkeletalMesh>(Requester))
			{
				Skeleton = Mesh->GetSkeleton();
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "RemoveDuplicateBones: Object supplied as requester (%ls) needs to be either Skeleton or SkeletalMesh", *GetFullNameSafe(Requester));
			}
		}

		// Additionally normalize all quaternions to be safe.
		for (int32 BoneIndex = 0; BoneIndex < GetRawBoneNum(); BoneIndex++)
		{
			RawRefBonePose[BoneIndex].NormalizeRotation();
		}

		const bool bRebuildNameMap = true;
		RebuildRefSkeleton(Skeleton, bRebuildNameMap);
	}

	// Make sure our arrays are in sync.
	checkSlow((RawRefBoneInfo.Num() == RawRefBonePose.Num()) && (RawRefBoneInfo.Num() == RawNameToIndexMap.Num()));
}

void FReferenceSkeleton::RebuildNameToIndexMap()
{
	// Start by clearing the current map.
	RawNameToIndexMap.Empty();

	// Then iterate over each bone, adding the name and bone index.
	const int32 NumBones = RawRefBoneInfo.Num();
	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const FName& BoneName = RawRefBoneInfo[BoneIndex].Name;
		if (BoneName != NAME_None)
		{
			RawNameToIndexMap.Add(BoneName, BoneIndex);
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "RebuildNameToIndexMap: Bone with no name detected for index: %d", BoneIndex);
		}
	}

	// Make sure we don't have duplicate bone names. This would be very bad.
	checkSlow(RawNameToIndexMap.Num() == NumBones);
}

SIZE_T FReferenceSkeleton::GetDataSize() const
{
	SIZE_T ResourceSize = 0;

	ResourceSize += RawRefBoneInfo.GetAllocatedSize();
	ResourceSize += RawRefBonePose.GetAllocatedSize();

	ResourceSize += FinalRefBoneInfo.GetAllocatedSize();
	ResourceSize += FinalRefBonePose.GetAllocatedSize();

	ResourceSize += RawNameToIndexMap.GetAllocatedSize();
	ResourceSize += FinalNameToIndexMap.GetAllocatedSize();

	return ResourceSize;
}

namespace
{
void ComposeAbsoluteTransforms(
	const FReferenceSkeleton& InRefSkeleton,
	TConstArrayView<FTransform> InBoneSpaceTransforms,
	TArray<FTransform>& OutAbsoluteTransforms
	)
{
	OutAbsoluteTransforms.SetNumUninitialized(InBoneSpaceTransforms.Num());

	for (int32 BoneIndex = 0; BoneIndex < InBoneSpaceTransforms.Num(); BoneIndex++)
	{
		const int32 ParentIndex = InRefSkeleton.GetParentIndex(BoneIndex);
		if (ParentIndex >= 0)
		{
			OutAbsoluteTransforms[BoneIndex] = InBoneSpaceTransforms[BoneIndex] * OutAbsoluteTransforms[ParentIndex];
		}
		else
		{
			OutAbsoluteTransforms[BoneIndex] = InBoneSpaceTransforms[BoneIndex];
		}
	}
}

FTransform ComposeSingleAbsoluteTransform(
	const int32 InBoneIndex,
	const FReferenceSkeleton& InRefSkeleton,
	TConstArrayView<FTransform> InBoneSpaceTransforms
	)
{
	FTransform AbsolutePose = InBoneSpaceTransforms[InBoneIndex];

	int32 ParentIndex = InRefSkeleton.GetParentIndex(InBoneIndex);

	while(ParentIndex != INDEX_NONE)
	{
		AbsolutePose = AbsolutePose * InBoneSpaceTransforms[ParentIndex];
		ParentIndex = InRefSkeleton.GetParentIndex(ParentIndex);
	}

	return AbsolutePose;
}


}


void FReferenceSkeleton::GetRawBoneAbsoluteTransforms(TArray<FTransform>& OutAbsoluteTransforms) const
{
	ComposeAbsoluteTransforms(*this, RawRefBonePose, OutAbsoluteTransforms);
}

void FReferenceSkeleton::GetBoneAbsoluteTransforms(TArray<FTransform>& OutAbsoluteTransforms) const
{
	ComposeAbsoluteTransforms(*this, FinalRefBonePose, OutAbsoluteTransforms);
}

FTransform FReferenceSkeleton::GetRawBoneAbsoluteTransform(const int32 InRawBoneIndex) const
{
	if (ensure(IsValidRawIndex(InRawBoneIndex)))
	{
		return ComposeSingleAbsoluteTransform(InRawBoneIndex, *this, RawRefBonePose);
	}

	return FTransform::Identity;
}

FTransform FReferenceSkeleton::GetBoneAbsoluteTransform(const int32 InBoneIndex) const
{
	if (ensure(IsValidIndex(InBoneIndex)))
	{
		return ComposeSingleAbsoluteTransform(InBoneIndex, *this, FinalRefBonePose);
	}

	return FTransform::Identity;
}

void FReferenceSkeleton::GetRawChildrenIndicesCached(const int32 BoneIndex, TArray<int32>& OutChildren) const
{
	const int32 LastBranchIndex = GetCachedEndOfBranchIndex(BoneIndex);
	if (LastBranchIndex == INDEX_NONE)
	{
		// no children (leaf bone)
		return;
	}
	
	for (int32 ChildBoneIndex = BoneIndex + 1; ChildBoneIndex <= LastBranchIndex; ChildBoneIndex++)
	{
		if (GetParentIndex(ChildBoneIndex) == BoneIndex)
		{
			OutChildren.Add(ChildBoneIndex);
		}
	}
}

void FReferenceSkeleton::GetRawChildrenIndicesRecursiveCached(const int32 BoneIndex, TArray<int32>& OutChildren) const
{
	const int32 LastBranchIndex = GetCachedEndOfBranchIndex(BoneIndex);
	if (LastBranchIndex == INDEX_NONE)
	{
		// no children (leaf bone)
		return;
	}
	
	for (int32 ChildBoneIndex = BoneIndex + 1; ChildBoneIndex <= LastBranchIndex; ChildBoneIndex++)
	{
		OutChildren.Add(ChildBoneIndex);
	}
}

struct FEnsureParentsExistScratchArea : public TThreadSingleton<FEnsureParentsExistScratchArea>
{
	TArray<bool> BoneExists;
};

void FReferenceSkeleton::EnsureParentsExist(TArray<FBoneIndexType>& InOutBoneSortedArray) const
{
	bool bWarn = false;
#if	!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Warn if we're getting bad data.
	// Bones are matched as int32, and a non found bone will be set to INDEX_NONE == -1
	// This should never happen, so if it does, something is wrong!
	bWarn = true;
#endif
	
	const int32 NumBones = GetNum();
	if (NumBones == 0)
	{
		if (bWarn && !InOutBoneSortedArray.IsEmpty())
		{
			UE_LOGF(LogAnimation, Log, "FReferenceSkeleton::EnsureParentsExist, RefSkeleton is empty. Removing all bones from InOutBoneSortedArray.");
		}
		InOutBoneSortedArray.Empty();
		return;
	}
	
	// remove invalid indices
	InOutBoneSortedArray.RemoveAll([bWarn, NumBones](const FBoneIndexType BoneIndex)
	{
		const bool bRemove = BoneIndex >= NumBones;	
		if (bWarn && bRemove)
		{
			UE_LOGF(LogAnimation, Log,
				"FReferenceSkeleton::EnsureParentsExist, invalid BoneIndex[%d / %d]. Removing bone from InOutBoneSortedArray.",
				BoneIndex, NumBones);	
		}
		return bRemove;
	});
	
	if (InOutBoneSortedArray.IsEmpty())
	{
		return;
	}
	
	// Iterate through existing array.
	TArray<bool>& BoneExists = FEnsureParentsExistScratchArea::Get().BoneExists;
	BoneExists.Reset();
	BoneExists.Init(false, NumBones);

	int32 i = 0;
	while (i < InOutBoneSortedArray.Num())
	{
		const int32 BoneIndex = InOutBoneSortedArray[i];

		// For the root bone, just move on.
		if (BoneIndex > 0)
		{			
			BoneExists[BoneIndex] = true;

			const int32 ParentIndex = GetParentIndex(BoneIndex);

			// If we do not have this parent in the array, we add it in this location, and leave 'i' where it is.
			// This can happen if somebody removes bones in the physics asset, then it will try add back in, and in the process, 
			// parent can be missing
			if (BoneExists.IsValidIndex(ParentIndex) && !BoneExists[ParentIndex])
			{
				InOutBoneSortedArray.InsertUninitialized(i);
				InOutBoneSortedArray[i] = ParentIndex;
				BoneExists[ParentIndex] = true;
			}
			// If parent was in array, just move on.
			else
			{
				i++;
			}
		}
		else
		{
			BoneExists[0] = true;
			i++;
		}
	}
}

void FReferenceSkeleton::EnsureParentsExistAndSort(TArray<FBoneIndexType>& InOutBoneUnsortedArray) const
{
	InOutBoneUnsortedArray.Sort();

	EnsureParentsExist(InOutBoneUnsortedArray);

	InOutBoneUnsortedArray.Sort();
}

int32 FReferenceSkeleton::GetChildrenInternal(int32 InParentBoneIndex, TArray<int32>& OutChildren, const bool bRaw) const
{
	OutChildren.Reset();

	const int32 NumBones = bRaw ? GetRawBoneNum() : GetNum();
	for (int32 ChildIndex = InParentBoneIndex + 1; ChildIndex < NumBones; ChildIndex++)
	{
		const int32 ParentIndex = bRaw ? GetRawParentIndex(ChildIndex) : GetParentIndex(ChildIndex);
		if (ParentIndex == InParentBoneIndex)
		{
			OutChildren.Add(ChildIndex);
		}
	}

	return OutChildren.Num();	
}

int32 FReferenceSkeleton::GetCachedEndOfBranchIndex(const int32 InBoneIndex) const
{
	if (!CachedEndOfBranchIndicesRaw.IsValidIndex(InBoneIndex))
	{
		return INDEX_NONE;
	}

	// already cached
	if (CachedEndOfBranchIndicesRaw[InBoneIndex] != BRANCH_CACHE_INVALID_INDEX)
	{
		return CachedEndOfBranchIndicesRaw[InBoneIndex];
	}
	
	const int32 NumBones = GetRawBoneNum();
	
	// if we're asking for the first or last bone, return the last bone  
	if (InBoneIndex == 0 || InBoneIndex + 1 >= NumBones)
	{
		CachedEndOfBranchIndicesRaw[InBoneIndex] = GetRawBoneNum()-1;
		return CachedEndOfBranchIndicesRaw[InBoneIndex];
	}
	
	const int32 StartParentIndex = GetParentIndex(InBoneIndex);
	int32 BoneIndex = InBoneIndex + 1;
	int32 ParentIndex = GetParentIndex(BoneIndex);

	// if next child bone's parent is less than or equal to StartParentIndex,
	// we are leaving the branch so no need to go further
	int32 BoneIndexAtEndOfBranch = BRANCH_CACHE_INVALID_INDEX;
	while (ParentIndex > StartParentIndex)
	{
		BoneIndexAtEndOfBranch = BoneIndex;
		BoneIndex++;
		if (BoneIndex >= NumBones)
		{
			break;
		}
		ParentIndex = GetParentIndex(BoneIndex);
	}

	// set once (outside of while loop above) to avoid potential race condition
	CachedEndOfBranchIndicesRaw[InBoneIndex] = BoneIndexAtEndOfBranch;
	
	return CachedEndOfBranchIndicesRaw[InBoneIndex];
}

int32 FReferenceSkeleton::GetDirectChildBones(int32 ParentBoneIndex, TArray<int32> & Children) const
{
	return GetChildrenInternal(ParentBoneIndex, Children, false);
}

int32 FReferenceSkeleton::GetRawDirectChildBones(int32 ParentBoneIndex, TArray<int32> & Children) const
{
	return GetChildrenInternal(ParentBoneIndex, Children, true);
}

FArchive & operator<<(FArchive & Ar, FReferenceSkeleton & F)
{
	Ar << F.RawRefBoneInfo;
	Ar << F.RawRefBonePose;

	if (Ar.UEVer() >= VER_UE4_REFERENCE_SKELETON_REFACTOR)
	{
		Ar << F.RawNameToIndexMap;
	}

	// Fix up any assets that don't have an INDEX_NONE parent for Bone[0]
	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_FIXUP_ROOTBONE_PARENT)
	{
		if ((F.RawRefBoneInfo.Num() > 0) && (F.RawRefBoneInfo[0].ParentIndex != INDEX_NONE))
		{
			F.RawRefBoneInfo[0].ParentIndex = INDEX_NONE;
		}
	}

	if (Ar.IsLoading())
	{
		F.FinalRefBoneInfo = F.RawRefBoneInfo;
		F.FinalRefBonePose = F.RawRefBonePose;
		F.FinalNameToIndexMap = F.RawNameToIndexMap;
	}

	return Ar;
}

