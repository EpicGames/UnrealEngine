// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/SkeletalMeshActor.h"
#include "Animation/AnimationAsset.h"
#include "Net/UnrealNetwork.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Engine/CollisionProfile.h"
#include "Logging/MessageLog.h"
#include "RenderUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshActor)

#define LOCTEXT_NAMESPACE "SkeletalMeshActor"


ASkeletalMeshActor::ASkeletalMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SkeletalMeshComponent = CreateDefaultSubobject<USkeletalMeshComponent>(TEXT("SkeletalMeshComponent0"));
	SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPose;
	// check BaseEngine.ini for profile setup
	SkeletalMeshComponent->SetCollisionProfileName(UCollisionProfile::PhysicsActor_ProfileName);
	RootComponent = SkeletalMeshComponent;

	bShouldDoAnimNotifies = true;
}

void ASkeletalMeshActor::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ASkeletalMeshActor, ReplicatedMesh);
	DOREPLIFETIME(ASkeletalMeshActor, ReplicatedPhysAsset);
	DOREPLIFETIME(ASkeletalMeshActor, ReplicatedMaterial0);
	DOREPLIFETIME(ASkeletalMeshActor, ReplicatedMaterial1);
}

/** Check SkeletalMeshActor for errors. */
#if WITH_EDITOR
void ASkeletalMeshActor::CheckForErrors()
{
	Super::CheckForErrors();

	if (SkeletalMeshComponent)
	{
		UPhysicsAsset * const PhysicsAsset = SkeletalMeshComponent->GetPhysicsAsset();
		if (!PhysicsAsset)
		{
			if (SkeletalMeshComponent->CastShadow
				&& SkeletalMeshComponent->bCastDynamicShadow
				&& IsStaticLightingAllowed())
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("DetailedInfo"), FText::FromString(GetDetailedInfo()));
				FMessageLog("MapCheck").PerformanceWarning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_SkelMeshActorNoPhysAsset", "SkeletalMeshActor '{DetailedInfo}' casts shadow but has no PhysicsAsset assigned.  The shadow will be low res and inefficient"), Arguments)))
					->AddToken(FMapErrorToken::Create(FMapErrors::SkelMeshActorNoPhysAsset));
			}
		}

		if (SkeletalMeshComponent->CastShadow
			&& SkeletalMeshComponent->bCastDynamicShadow
			&& SkeletalMeshComponent->IsRegistered()
			&& SkeletalMeshComponent->Bounds.SphereRadius > 2000.0f
			&& IsStaticLightingAllowed())
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("DetailedInfo"), FText::FromString(GetDetailedInfo()));
			// Large shadow casting objects that create preshadows will cause a massive performance hit, since preshadows are meant for small shadow casters.
			FMessageLog("MapCheck").PerformanceWarning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_ActorLargeShadowCaster", "{DetailedInfo} : Large actor casts a shadow and will cause an extreme performance hit unless shadow casting is disabled"), Arguments)))
				->AddToken(FMapErrorToken::Create(FMapErrors::ActorLargeShadowCaster));
		}

		if (SkeletalMeshComponent->GetSkeletalMeshAsset() == NULL)
		{
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_SkeletalMeshNull", "Skeletal mesh actor has NULL SkeletalMesh property")))
				->AddToken(FMapErrorToken::Create(FMapErrors::SkeletalMeshNull));
		}
	}
	else
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_SkeletalMeshComponent", "Skeletal mesh actor has NULL SkeletalMeshComponent property")))
			->AddToken(FMapErrorToken::Create(FMapErrors::SkeletalMeshComponent));
	}
}
#endif

FString ASkeletalMeshActor::GetDetailedInfoInternal() const
{
	FString Result;

	if (SkeletalMeshComponent)
	{
		Result = SkeletalMeshComponent->GetDetailedInfoInternal();
	}
	else
	{
		Result = TEXT("No_SkeletalMeshComponent");
	}

	return Result;
}

void ASkeletalMeshActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();
	// grab the current mesh for replication
	if (GetLocalRole() == ROLE_Authority && SkeletalMeshComponent)
	{
		ReplicatedMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	}

	// Unfix bodies flagged as 'full anim weight'
	if (SkeletalMeshComponent)
	{
		ReplicatedPhysAsset = SkeletalMeshComponent->GetPhysicsAsset();
	}
}

void ASkeletalMeshActor::OnRep_ReplicatedMesh()
{
	SkeletalMeshComponent->SetSkeletalMesh(ReplicatedMesh);
}

void ASkeletalMeshActor::OnRep_ReplicatedPhysAsset()
{
	SkeletalMeshComponent->SetPhysicsAsset(ReplicatedPhysAsset);
}

void ASkeletalMeshActor::OnRep_ReplicatedMaterial0()
{
	SkeletalMeshComponent->SetMaterial(0, ReplicatedMaterial0);
}

void ASkeletalMeshActor::OnRep_ReplicatedMaterial1()
{
	SkeletalMeshComponent->SetMaterial(1, ReplicatedMaterial1);
}

#if WITH_EDITOR

bool ASkeletalMeshActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (SkeletalMeshComponent)
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			Objects.Add(SkeletalMeshComponent->GetSkeletalMeshAsset());
		}

		if (SkeletalMeshComponent->GetAnimationMode() == EAnimationMode::Type::AnimationSingleNode &&
			SkeletalMeshComponent->AnimationData.AnimToPlay)
		{
			Objects.Add(SkeletalMeshComponent->AnimationData.AnimToPlay);
		}
	}

	return true;
}

void ASkeletalMeshActor::EditorReplacedActor(AActor* OldActor)
{
	Super::EditorReplacedActor(OldActor);

	if (ASkeletalMeshActor * OldSkelMeshActor = Cast<ASkeletalMeshActor>(OldActor))
	{
		// if no skeletal mesh set, take one from previous actor
		if (SkeletalMeshComponent && OldSkelMeshActor->SkeletalMeshComponent &&
			SkeletalMeshComponent->GetSkeletalMeshAsset() == NULL)
		{
			SkeletalMeshComponent->SetSkeletalMesh(OldSkelMeshActor->SkeletalMeshComponent->GetSkeletalMeshAsset());
		}
	}
}

void ASkeletalMeshActor::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if (GetLinkerUEVersion() < VER_UE4_REMOVE_SKELETALPHYSICSACTOR)
	{
		static FName SkeletalPhysicsActor_NAME("SkeletalPhysicsActor");
		static FName KAsset_NAME("KAsset");

		if (OldClassName == SkeletalPhysicsActor_NAME || OldClassName == KAsset_NAME)
		{
			SkeletalMeshComponent->KinematicBonesUpdateType = EKinematicBonesUpdateToPhysics::SkipSimulatingBones;
			SkeletalMeshComponent->BodyInstance.bSimulatePhysics = true;
			SkeletalMeshComponent->bBlendPhysics = true;

			bAlwaysRelevant = true;
			SetReplicatingMovement(true);
			SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
			bReplicates = true;
		}
	}
}
#endif

#undef LOCTEXT_NAMESPACE

