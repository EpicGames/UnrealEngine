// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/SkeletalMeshVertexAttribute.h"

#include "Rendering/SkeletalMeshAttributeVertexBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshVertexAttribute)


bool FSkeletalMeshVertexAttributeInfo::IsEnabledForRender() const
{
	return EnabledForRender.GetValue();
}


#if WITH_EDITOR
FArchive& operator<<(FArchive& Ar, FSkeletalMeshModelVertexAttribute& Attribute)
{
	Ar << Attribute.DataType;
	Ar << Attribute.Values;
	Ar << Attribute.ComponentCount;
	return Ar;
}
#endif


FSkeletalMeshVertexAttributeRenderData::~FSkeletalMeshVertexAttributeRenderData()
{
	for (const TTuple<FName, FSkeletalMeshAttributeVertexBuffer*>& BufferItem: Buffers)
	{
		delete BufferItem.Value;
	}
}


bool FSkeletalMeshVertexAttributeRenderData::AddAttribute(
	const FName InName,
	const ESkeletalMeshVertexAttributeDataType InDataType,
	const int32 InNumVertices,
	const int32 InComponentCount,
	const TArray<float>& InValues
	)
{
	check(InComponentCount >= 1 && InComponentCount <= 4);
	
	FSkeletalMeshAttributeVertexBuffer* Buffer;
	if (Buffers.Contains(InName))
	{
		Buffer = Buffers[InName];
	}
	else
	{
		Buffer = Buffers.Add(InName, new FSkeletalMeshAttributeVertexBuffer);
	}

	EPixelFormat PixelFormat = PF_Unknown;
	switch(InDataType)
	{
	case ESkeletalMeshVertexAttributeDataType::NUInt8:
		
		PixelFormat = PF_R8;
		break;
	case ESkeletalMeshVertexAttributeDataType::Float:
		PixelFormat = PF_R32_FLOAT;
		break;
	case ESkeletalMeshVertexAttributeDataType::HalfFloat:
		PixelFormat = PF_R16F;
		break;
	}

	Buffer->Init(PixelFormat, InNumVertices, InComponentCount, InValues);
	return true;
}


FSkeletalMeshAttributeVertexBuffer* FSkeletalMeshVertexAttributeRenderData::GetAttributeBuffer(FName InName) const
{
	FSkeletalMeshAttributeVertexBuffer* const* BufferPtr = Buffers.Find(InName);
	return BufferPtr ? *BufferPtr : nullptr;
}

void FSkeletalMeshVertexAttributeRenderData::InitResources()
{
	check(IsInGameThread());
	for (const TTuple<FName, FSkeletalMeshAttributeVertexBuffer*>& BufferItem: Buffers)
	{
		BeginInitResource(BufferItem.Value);
	}
}

void FSkeletalMeshVertexAttributeRenderData::ReleaseResources()
{
	check(IsInGameThread());
	for (const TTuple<FName, FSkeletalMeshAttributeVertexBuffer*>& BufferItem: Buffers)
	{
		BeginReleaseResource(BufferItem.Value);
	}
}


void FSkeletalMeshVertexAttributeRenderData::CleanUp()
{
	for (const TTuple<FName, FSkeletalMeshAttributeVertexBuffer*>& BufferItem: Buffers)
	{
		BufferItem.Value->CleanUp();
	}
}


int32 FSkeletalMeshVertexAttributeRenderData::GetResourceSize() const
{
	int32 Size = 0;
	for (const TTuple<FName, FSkeletalMeshAttributeVertexBuffer*>& BufferItem: Buffers)
	{
		Size += BufferItem.Value->GetResourceSize();
	}
	return Size;
}

namespace SkeletalMeshResourceNames
{
	static const FName SkeletalMeshAttributeVertexBufferName("FSkeletalMeshVertexAttributeRenderData.FSkeletalMeshAttributeVertexBuffer");
} // SkeletalMeshResourceNames

void FSkeletalMeshVertexAttributeRenderData::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		int32 Count;
		Ar << Count;
		Buffers.Reset();
		for (int32 Index = 0; Index < Count && !Ar.IsError(); Index++)
		{
			FName Name;
			Ar << Name;
			FSkeletalMeshAttributeVertexBuffer* Buffer = new FSkeletalMeshAttributeVertexBuffer;
			Buffer->Serialize(Ar);
			Buffers.Add(Name, Buffer);
			Buffer->SetResourceName(SkeletalMeshResourceNames::SkeletalMeshAttributeVertexBufferName);
		}
	}
	else
	{
		int32 Count = Buffers.Num();
		Ar << Count;
		for (const TTuple<FName, FSkeletalMeshAttributeVertexBuffer*>& BufferItem: Buffers)
		{
			FName Name = BufferItem.Key;
			Ar << Name;
			BufferItem.Value->Serialize(Ar);
		}
	}
}
