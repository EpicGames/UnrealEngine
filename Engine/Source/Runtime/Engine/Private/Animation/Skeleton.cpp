// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Skeleton.cpp: Skeleton features
=============================================================================*/ 

#include "Animation/Skeleton.h"

#include "AnimationSequenceCompiler.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Components/SkinnedMeshComponent.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "Engine/AssetUserData.h"
#include "Modules/ModuleManager.h"
#include "Engine/DataAsset.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimSequence.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Animation/BlendProfile.h"
#include "Engine/SkinnedAsset.h"
#include "Logging/MessageLog.h"
#include "ComponentReregisterContext.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/ScopedSlowTask.h"
#include "Animation/AnimBlueprint.h"
#include "UObject/AnimObjectVersion.h"
#include "EngineUtils.h"
#include "ObjectCacheContext.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Skeleton)

#define LOCTEXT_NAMESPACE "Skeleton"
#define ROOT_BONE_PARENT	INDEX_NONE


#if WITH_EDITOR
const FName USkeleton::AnimNotifyTag = FName("AnimNotifyList");
const FString USkeleton::AnimNotifyTagDelimiter = TEXT(";");

const FName USkeleton::AnimSyncMarkerTag = FName("AnimSyncMarkerList");
const FString USkeleton::AnimSyncMarkerTagDelimiter = TEXT(";");

const FName USkeleton::CurveNameTag = FName("CurveNameList");
const FString USkeleton::CurveTagDelimiter = TEXT(";");

const FName USkeleton::AttributeTag = FName("AttributeList");

const FName USkeleton::CompatibleSkeletonsNameTag = FName("CompatibleSkeletonList");
const FString USkeleton::CompatibleSkeletonsTagDelimiter = TEXT(";");
#endif 

#if WITH_EDITORONLY_DATA
FAreAllSkeletonsCompatible USkeleton::AreAllSkeletonsCompatibleDelegate;
#endif

const FName FAnimSlotGroup::DefaultGroupName = FName("DefaultGroup");
const FName FAnimSlotGroup::DefaultSlotName = FName("DefaultSlot");

TAutoConsoleVariable<bool> CVarAllowIncompatibleSkeletalMeshMerge(TEXT("a.Skeleton.AllowIncompatibleSkeletalMeshMerge"), 0, TEXT("When importing or otherwise merging in skeletal mesh bones, allow 'incompatible' hierarchies with bone insertions."));

void SerializeReferencePose(FArchive& Ar, FReferencePose& P, UObject* Outer)
{
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);

	Ar << P.PoseName;
	Ar << P.ReferencePose;
#if WITH_EDITORONLY_DATA
	//TODO: we should use strip flags but we need to rev the serialization version
	if (!Ar.IsCooking() && (!Ar.IsLoading() || !Outer->GetOutermost()->bIsCookedForEditor))
	{
		if (Ar.IsLoading() && Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::ChangeRetargetSourceReferenceToSoftObjectPtr)
		{
			USkeletalMesh* SourceMesh = nullptr;
			Ar << SourceMesh;
			P.SourceReferenceMesh = SourceMesh;
		}
		else
		{
			// Scope the soft pointer serialization so we can tag it as editor only
			FName PackageName;
			FName PropertyName;
			ESoftObjectPathCollectType CollectType = ESoftObjectPathCollectType::AlwaysCollect;
			ESoftObjectPathSerializeType SerializeType = ESoftObjectPathSerializeType::AlwaysSerialize;
			FSoftObjectPathThreadContext& ThreadContext = FSoftObjectPathThreadContext::Get();
			ThreadContext.GetSerializationOptions(PackageName, PropertyName, CollectType, SerializeType);
			FSoftObjectPathSerializationScope SerializationScope(PackageName, PropertyName, ESoftObjectPathCollectType::EditorOnlyCollect, SerializeType);
			Ar << P.SourceReferenceMesh;
		}
	}
#endif
}

const TCHAR* SkipPrefix(const FString& InName)
{
	const int32 PrefixLength = VirtualBoneNameHelpers::VirtualBonePrefix.Len();
	check(InName.Len() > PrefixLength);
	return &InName[PrefixLength];
}

namespace VirtualBoneNameHelpers
{
	const FString VirtualBonePrefix("VB ");

	FString AddVirtualBonePrefix(const FString& InName)
	{
		return VirtualBonePrefix + InName;
	}

	FName RemoveVirtualBonePrefix(const FString& InName)
	{
		return FName(SkipPrefix(InName));
	}

	bool CheckVirtualBonePrefix(const FString& InName)
	{
		return InName.StartsWith(VirtualBonePrefix);
	}
}

#if WITH_EDITORONLY_DATA
bool USkeleton::IsCompatibleForEditor(const USkeleton* InSkeleton) const
{
	return IsCompatibleForEditor(FAssetData(InSkeleton));
}

bool USkeleton::IsCompatibleForEditor(const FAssetData& AssetData, const TCHAR* InTag) const
{
	if(AssetData.GetClass() == USkeleton::StaticClass())
	{
		return IsCompatibleForEditor(AssetData.GetExportTextName());
	}
	else
	{
		return IsCompatibleForEditor(AssetData.GetTagValueRef<FString>(InTag));
	}
}

bool USkeleton::IsCompatibleForEditor(const FString& SkeletonAssetString) const
{
	// First check against itself.
	TStringBuilder<128> SkeletonStringBuilder;
	FAssetData(this, FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering).GetExportTextName(SkeletonStringBuilder);
	const TCHAR* SkeletonString = SkeletonStringBuilder.ToString();
	if (SkeletonAssetString == SkeletonString)
	{
		return true;
	}

	// Let the global delegate override any per-skeleton settings
	if(AreAllSkeletonsCompatibleDelegate.IsBound() && AreAllSkeletonsCompatibleDelegate.Execute())
	{
		return true;
	}

	// Now check against the list of compatible skeletons and see if we're dealing with the same asset.
	if(CompatibleSkeletons.Num() > 0)
	{
		const FSoftObjectPath InPath(SkeletonAssetString);
		for (const TSoftObjectPtr<USkeleton>& CompatibleSkeleton : CompatibleSkeletons)
		{
			if (CompatibleSkeleton.ToSoftObjectPath() == InPath)
			{
				return true;
			}
		}
	}

	// Check if the other skeleton is compatible with this via the asset registry
	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	constexpr bool bIncludeOnlyOnDiskAssets = true;
	const FAssetData SkeletonAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(*SkeletonAssetString), bIncludeOnlyOnDiskAssets);
	const FString TagValue = SkeletonAssetData.GetTagValueRef<FString>(USkeleton::CompatibleSkeletonsNameTag);
	if (!TagValue.IsEmpty())
	{
		TArray<FString> OtherCompatibleSkeletons;
		if (TagValue.ParseIntoArray(OtherCompatibleSkeletons, *USkeleton::CompatibleSkeletonsTagDelimiter, true) > 0)
		{
			for (const FString& OtherCompatibleSkeleton : OtherCompatibleSkeletons)
			{
				if (OtherCompatibleSkeleton == SkeletonString)
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool USkeleton::ShouldFilterAsset(const FAssetData& InAssetData, const TCHAR* InTag) const
{
	return !IsCompatibleForEditor(InAssetData, InTag);
}

void USkeleton::GetCompatibleSkeletonAssets(TArray<FAssetData>& OutAssets) const
{
	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	// If we are compatible with all, then just add all skeletons
	if(AreAllSkeletonsCompatibleDelegate.IsBound() && AreAllSkeletonsCompatibleDelegate.Execute())
	{
		TArray<FAssetData> AllSkeletons;
		FARFilter ARFilter;
		ARFilter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
		AssetRegistry.GetAssets(ARFilter, AllSkeletons);

		OutAssets.Append(AllSkeletons);
	}
	else
	{
		// Always add 'this'
		FAssetData AssetDataThis(this);
		OutAssets.Add(AssetDataThis);
		
		// Add skeletons in our compatibility list
		for(TSoftObjectPtr<USkeleton> CompatibleSkeleton : CompatibleSkeletons)
		{
			const FAssetData SkeletonAssetData = AssetRegistry.GetAssetByObjectPath(CompatibleSkeleton.ToSoftObjectPath());
			if(SkeletonAssetData.IsValid())
			{
				OutAssets.Add(SkeletonAssetData);
			}
		}

		// Add skeletons where we are listed in their compatibility list
		{
			TArray<FAssetData> SkeletonsCompatibleWithThis;
			FARFilter ARFilter;
			ARFilter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
			ARFilter.TagsAndValues.Add(CompatibleSkeletonsNameTag, AssetDataThis.GetExportTextName());
			AssetRegistry.GetAssets(ARFilter, SkeletonsCompatibleWithThis);

			OutAssets.Append(SkeletonsCompatibleWithThis);
		}
	}
}

void USkeleton::GetCompatibleAssets(UClass* AssetClass, const TCHAR* InTag, TArray<FAssetData>& OutAssets) const
{
	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	
	if(AreAllSkeletonsCompatibleDelegate.IsBound() && AreAllSkeletonsCompatibleDelegate.Execute())
	{
		// Compatible with all, so return all assets of type
		FARFilter ARFilter;
		ARFilter.ClassPaths.Add(AssetClass->GetClassPathName());
		ARFilter.bRecursiveClasses = true;
		AssetRegistry.GetAssets(ARFilter, OutAssets);
	}
	else
	{
		FARFilter ARFilter;
		ARFilter.ClassPaths.Add(AssetClass->GetClassPathName());
		ARFilter.bRecursiveClasses = true;

		TArray<FAssetData> CompatibleSkeletonAssets;
		GetCompatibleSkeletonAssets(CompatibleSkeletonAssets);

		for (const FAssetData& CompatibleSkeleton : CompatibleSkeletonAssets)
		{
			ARFilter.TagsAndValues.Add(InTag, CompatibleSkeleton.GetExportTextName());
		}

		AssetRegistry.GetAssets(ARFilter, OutAssets);
	}
}

#endif

void USkeleton::AddCompatibleSkeleton(const USkeleton* SourceSkeleton)
{
	CompatibleSkeletons.AddUnique(const_cast<USkeleton*>(SourceSkeleton));
}

void USkeleton::AddCompatibleSkeletonSoft(const TSoftObjectPtr<USkeleton>& SourceSkeleton)
{
	CompatibleSkeletons.AddUnique(SourceSkeleton);
}

void USkeleton::RemoveCompatibleSkeleton(const USkeleton* SourceSkeleton)
{
	CompatibleSkeletons.Remove(const_cast<USkeleton*>(SourceSkeleton));
}

void USkeleton::RemoveCompatibleSkeleton(const TSoftObjectPtr<USkeleton>& SourceSkeleton)
{
	CompatibleSkeletons.Remove(SourceSkeleton);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USkeleton::USkeleton(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CachedSoftObjectPtr = TSoftObjectPtr<USkeleton>(this);

#if WITH_EDITORONLY_DATA
	PreviewForwardAxis = EAxis::Y;
#endif

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FCoreUObjectDelegates::OnPackageReloaded.AddStatic(&USkeleton::HandlePackageReloaded);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void USkeleton::BeginDestroy()
{
#if WITH_EDITOR
	UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation({this});
#endif // WITH_EDITOR
	
	Super::BeginDestroy();
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
	}
}

void USkeleton::PostInitProperties()
{
	Super::PostInitProperties();

	// this gets called after constructor, and this data can get
	// serialized back if this already has Guid
	if (!IsTemplate())
	{
		RegenerateGuid();
	}
}

bool USkeleton::IsPostLoadThreadSafe() const
{
	return WITH_EDITORONLY_DATA == 0;
}

void USkeleton::PostLoad()
{
	Super::PostLoad();
	LLM_SCOPE(ELLMTag::Animation);

#if WITH_EDITORONLY_DATA
	if( GetLinker() && (GetLinker()->UEVer() < VER_UE4_REFERENCE_SKELETON_REFACTOR) )
	{
		// Convert RefLocalPoses & BoneTree to FReferenceSkeleton
		ConvertToFReferenceSkeleton();
	}
#endif

	// catch any case if guid isn't valid
	check(Guid.IsValid());

	// Cleanup CompatibleSkeletons for convenience. This basically removes any soft object pointers that has an invalid soft object name.
	CompatibleSkeletons = CompatibleSkeletons.FilterByPredicate([](const TSoftObjectPtr<USkeleton>& Skeleton)
	{
		return Skeleton.ToSoftObjectPath().IsValid();
	});

#if WITH_EDITORONLY_DATA
	if(GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimationRemoveSmartNames)
	{
		// Move curve metadata over to asset user data
		UAnimCurveMetaData* AnimCurveMetaData = GetOrCreateCurveMetaDataObject();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for(const TPair<FName, FSmartNameMapping>& NameMappingPair : SmartNames_DEPRECATED.NameMappings)
		{
			for(const TPair<FName, FCurveMetaData>& NameMetaDataPair : NameMappingPair.Value.CurveMetaDataMap)
			{
				AnimCurveMetaData->CurveMetaData.Add(NameMetaDataPair.Key, NameMetaDataPair.Value);
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		// Ensure we have curve metadata, even if empty, so we can correctly pick up older objects in the asset registry
		GetOrCreateCurveMetaDataObject();
	}
#endif

	// refresh linked bone indices
	RefreshSkeletonMetaData();

}

void USkeleton::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		// regenerate Guid
		RegenerateGuid();
	}
}

void USkeleton::Serialize( FArchive& Ar )
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	DECLARE_SCOPE_CYCLE_COUNTER( TEXT("USkeleton::Serialize"), STAT_Skeleton_Serialize, STATGROUP_LoadTime );

	Super::Serialize(Ar);

	if( Ar.UEVer() >= VER_UE4_REFERENCE_SKELETON_REFACTOR )
	{
		Ar << ReferenceSkeleton;
	}

	if (Ar.UEVer() >= VER_UE4_FIX_ANIMATIONBASEPOSE_SERIALIZATION)
	{
		// Load Animation RetargetSources
		if (Ar.IsLoading())
		{
			int32 NumOfRetargetSources;
			Ar << NumOfRetargetSources;

			FName RetargetSourceName;
			FReferencePose RetargetSource;
			AnimRetargetSources.Empty(NumOfRetargetSources);
			for (int32 Index=0; Index<NumOfRetargetSources; ++Index)
			{
				Ar << RetargetSourceName;
				SerializeReferencePose(Ar, RetargetSource, this);

				AnimRetargetSources.Add(RetargetSourceName, RetargetSource);
			}
		}
		else 
		{
			int32 NumOfRetargetSources = AnimRetargetSources.Num();
			Ar << NumOfRetargetSources;

			for (auto Iter = AnimRetargetSources.CreateIterator(); Iter; ++Iter)
			{
				Ar << Iter.Key();
				SerializeReferencePose(Ar, Iter.Value(), this);
			}
		}
	}
	else
	{
		// this is broken, but we have to keep it to not corrupt content. 
		for (auto Iter = AnimRetargetSources.CreateIterator(); Iter; ++Iter)
		{
			Ar << Iter.Key();
			SerializeReferencePose(Ar, Iter.Value(), this);
		}
	}

	if (Ar.UEVer() < VER_UE4_SKELETON_GUID_SERIALIZATION)
	{
		UE_LOGF(LogAnimation, Warning, "Skeleton '%ls' has not been saved since version 'VER_UE4_SKELETON_GUID_SERIALIZATION' This asset will not cook deterministically until it is resaved.", *GetPathName());
		RegenerateGuid();
	}
	else
	{
		Ar << Guid;
	}

#if WITH_EDITORONLY_DATA
	// Drain the legacy FSmartNameContainer payload from source assets predating
	// RemovedSmartNameContainerPayload. Editor-only: cooked runtime never loads stale packages.
	if (Ar.UEVer() >= VER_UE4_SKELETON_ADD_SMARTNAMES)
	{
		Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID)
				< FUE5ReleaseStreamObjectVersion::RemovedSmartNameContainerPayload)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SmartNames_DEPRECATED.Serialize(Ar, IsTemplate());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif

	// Build look up table between Slot nodes and their Group.
	if(Ar.UEVer() < VER_UE4_FIX_SLOT_NAME_DUPLICATION)
	{
		// In older assets we may have duplicates, remove these while building the map.
		BuildSlotToGroupMap(true);
	}
	else
	{
		BuildSlotToGroupMap();
	}

#if WITH_EDITORONLY_DATA
	if (Ar.UEVer() < VER_UE4_SKELETON_ASSET_PROPERTY_TYPE_CHANGE)
	{
		PreviewAttachedAssetContainer.SaveAttachedObjectsFromDeprecatedProperties();
	}
#endif

	if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::StoreMarkerNamesOnSkeleton)
	{
		FStripDataFlags StripFlags(Ar);
		if (!StripFlags.IsEditorDataStripped())
		{
			Ar << ExistingMarkerNames;
		}
	}

	if (Ar.IsLoading())
	{
		const bool bRebuildNameMap = false;
		ReferenceSkeleton.RebuildRefSkeleton(this, bRebuildNameMap);
	}
}

#if WITH_EDITOR
void USkeleton::PreEditUndo()
{
	// Undoing so clear cached data as it will now be stale
	ClearCacheData();
}

void USkeleton::PostEditUndo()
{
	Super::PostEditUndo();

	//If we were undoing virtual bone changes then we need to handle stale cache data
	// Cached data is cleared in PreEditUndo to make sure it is done before any object hits their PostEditUndo
	HandleVirtualBoneChanges();
}
#endif // WITH_EDITOR

/** Remove this function when VER_UE4_REFERENCE_SKELETON_REFACTOR is removed. */
void USkeleton::ConvertToFReferenceSkeleton()
{
#if WITH_EDITORONLY_DATA
	FReferenceSkeletonModifier RefSkelModifier(ReferenceSkeleton, this);

	check( BoneTree.Num() == RefLocalPoses_DEPRECATED.Num() );

	const int32 NumRefBones = RefLocalPoses_DEPRECATED.Num();
	ReferenceSkeleton.Empty();

	for(int32 BoneIndex=0; BoneIndex<NumRefBones; BoneIndex++)
	{
		const FBoneNode& BoneNode = BoneTree[BoneIndex];
		FMeshBoneInfo BoneInfo(BoneNode.Name_DEPRECATED, BoneNode.Name_DEPRECATED.ToString(), BoneNode.ParentIndex_DEPRECATED);
		const FTransform& BoneTransform = RefLocalPoses_DEPRECATED[BoneIndex];

		// All should be good. Parents before children, no duplicate bones?
		RefSkelModifier.Add(BoneInfo, BoneTransform);
	}

	// technically here we should call 	RefershAllRetargetSources(); but this is added after 
	// VER_UE4_REFERENCE_SKELETON_REFACTOR, this shouldn't be needed. It shouldn't have any 
	// AnimatedRetargetSources
	ensure (AnimRetargetSources.Num() == 0);
#endif
}


#if WITH_EDITOR

bool USkeleton::RemoveMarkerName(FName MarkerName)
{
	if(ExistingMarkerNames.Contains(MarkerName))
	{
		Modify();
	}

	return ExistingMarkerNames.Remove(MarkerName) != 0;
}

bool USkeleton::RenameMarkerName(FName InOldName, FName InNewName)
{
	if(ExistingMarkerNames.Contains(InOldName))
	{
		Modify();
	}

	if(ExistingMarkerNames.Contains(InNewName))
	{
		return ExistingMarkerNames.Remove(InOldName) != 0;
	}

	for(FName& MarkerName : ExistingMarkerNames)
	{
		if(MarkerName == InOldName)
		{
			MarkerName = InNewName;
			return true;
		}
	}

	return false;
}

#endif

bool USkeleton::DoesParentChainMatch(int32 StartBoneIndex, const USkinnedAsset* InSkinnedAsset) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeleton::DoesParentChainMatch);
	const FReferenceSkeleton& SkeletonRefSkel = ReferenceSkeleton;
	const FReferenceSkeleton& MeshRefSkel = InSkinnedAsset->GetRefSkeleton();

	// if start is root bone
	if ( StartBoneIndex == 0 )
	{
		// verify name of root bone matches
		return (SkeletonRefSkel.GetBoneName(0) == MeshRefSkel.GetBoneName(0));
	}

	int32 SkeletonBoneIndex = StartBoneIndex;
	// If skeleton bone is not found in mesh, fail.
	int32 MeshBoneIndex = MeshRefSkel.FindBoneIndex( SkeletonRefSkel.GetBoneName(SkeletonBoneIndex) );
	if( MeshBoneIndex == INDEX_NONE )
	{
		return false;
	}
	do
	{
		// verify if parent name matches
		int32 ParentSkeletonBoneIndex = SkeletonRefSkel.GetParentIndex(SkeletonBoneIndex);
		int32 ParentMeshBoneIndex = MeshRefSkel.GetParentIndex(MeshBoneIndex);

		// if one of the parents doesn't exist, make sure both end. Otherwise fail.
		if( (ParentSkeletonBoneIndex == INDEX_NONE) || (ParentMeshBoneIndex == INDEX_NONE) )
		{
			return (ParentSkeletonBoneIndex == ParentMeshBoneIndex);
		}

		// If parents are not named the same, fail.
		if( SkeletonRefSkel.GetBoneName(ParentSkeletonBoneIndex) != MeshRefSkel.GetBoneName(ParentMeshBoneIndex) )
		{
			return false;
		}

		// move up
		SkeletonBoneIndex = ParentSkeletonBoneIndex;
		MeshBoneIndex = ParentMeshBoneIndex;
	} while ( true );
}

bool USkeleton::IsCompatibleMesh(const USkinnedAsset* InSkinnedAsset, bool bDoParentChainCheck) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeleton::IsCompatibleMesh);
	// at least % of bone should match 
	int32 NumOfBoneMatches = 0;

	const FReferenceSkeleton& SkeletonRefSkel = ReferenceSkeleton;
	const FReferenceSkeleton& MeshRefSkel = InSkinnedAsset->GetRefSkeleton();
	const int32 NumBones = MeshRefSkel.GetRawBoneNum();

	// first ensure the parent exists for each bone
	for (int32 MeshBoneIndex=0; MeshBoneIndex<NumBones; MeshBoneIndex++)
	{
		FName MeshBoneName = MeshRefSkel.GetBoneName(MeshBoneIndex);
		// See if Mesh bone exists in Skeleton.
		int32 SkeletonBoneIndex = SkeletonRefSkel.FindBoneIndex( MeshBoneName );

		// if found, increase num of bone matches count
		if( SkeletonBoneIndex != INDEX_NONE )
		{
			++NumOfBoneMatches;

			// follow the parent chain to verify the chain is same
			if(bDoParentChainCheck && !DoesParentChainMatch(SkeletonBoneIndex, InSkinnedAsset))
			{
				UE_LOGF(LogAnimation, Verbose, "%ls : Hierarchy does not match.", *MeshBoneName.ToString());
				return false;
			}
		}
		else
		{
			int32 CurrentBoneId = MeshBoneIndex;
			// if not look for parents that matches
			while (SkeletonBoneIndex == INDEX_NONE && CurrentBoneId != INDEX_NONE)
			{
				// find Parent one see exists
				const int32 ParentMeshBoneIndex = MeshRefSkel.GetParentIndex(CurrentBoneId);
				if ( ParentMeshBoneIndex != INDEX_NONE )
				{
					// @TODO: make sure RefSkeleton's root ParentIndex < 0 if not, I'll need to fix this by checking TreeBoneIdx
					FName ParentBoneName = MeshRefSkel.GetBoneName(ParentMeshBoneIndex);
					SkeletonBoneIndex = SkeletonRefSkel.FindBoneIndex(ParentBoneName);
				}

				// root is reached
				if( ParentMeshBoneIndex == 0 )
				{
					break;
				}
				else
				{
					CurrentBoneId = ParentMeshBoneIndex;
				}
			}

			// still no match, return false, no parent to look for
			if( SkeletonBoneIndex == INDEX_NONE )
			{
				UE_LOGF(LogAnimation, Verbose, "%ls : Missing joint on skeleton.  Make sure to assign to the skeleton.", *MeshBoneName.ToString());
				return false;
			}

			// second follow the parent chain to verify the chain is same
			if (bDoParentChainCheck && !DoesParentChainMatch(SkeletonBoneIndex, InSkinnedAsset))
			{
				UE_LOGF(LogAnimation, Verbose, "%ls : Hierarchy does not match.", *MeshBoneName.ToString());
				return false;
			}
		}
	}

	// originally we made sure at least matches more than 50% 
	// but then follower components can't play since they're only partial
	// if the hierarchy matches, and if it's more then 1 bone, we allow
	return (NumOfBoneMatches > 0);
}

void USkeleton::ClearCacheData()
{
	{
		UE::TWriteScopeLock Lock(SkinnedAssetLinkupCacheLock);
		SkinnedAssetLinkupCache.Empty();
	}
}

const FSkeletonToMeshLinkup& USkeleton::FindOrAddMeshLinkupData(const USkinnedAsset* InSkinnedAsset)
{
	const TUniquePtr<FSkeletonToMeshLinkup>* SkeletonToMeshLinkupPtr = nullptr;

	{
		UE::TReadScopeLock Lock(SkinnedAssetLinkupCacheLock);
		SkeletonToMeshLinkupPtr = SkinnedAssetLinkupCache.Find(InSkinnedAsset);
	}

	return (SkeletonToMeshLinkupPtr != nullptr)
		? *SkeletonToMeshLinkupPtr->Get()
		: AddMeshLinkupData(InSkinnedAsset);
}

const FSkeletonToMeshLinkup& USkeleton::AddMeshLinkupData(const USkinnedAsset* InSkinnedAsset)
{
	TUniquePtr<FSkeletonToMeshLinkup> TmpLinkup = MakeUnique<FSkeletonToMeshLinkup>();
	BuildLinkupData(InSkinnedAsset, *TmpLinkup.Get());

	const TObjectKey<USkinnedAsset> SkinnedAssetKey(InSkinnedAsset);
	const uint32 SkinnedAssetHash = GetTypeHash(SkinnedAssetKey);

	{
		UE::TWriteScopeLock Lock(SkinnedAssetLinkupCacheLock);
		if (const TUniquePtr<FSkeletonToMeshLinkup>* SkeletonToMeshLinkupPtr = SkinnedAssetLinkupCache.FindByHash(SkinnedAssetHash, SkinnedAssetKey))
		{
			// While we were building the linkup data, another thread beat us to it
			// Discard the work we did to avoid freeing memory other threads might now be using
			return *SkeletonToMeshLinkupPtr->Get();
		}

		return *SkinnedAssetLinkupCache.AddByHash(SkinnedAssetHash, SkinnedAssetKey, MoveTemp(TmpLinkup)).Get();
	}
}

void USkeleton::RemoveLinkup(const USkinnedAsset* InSkinnedAsset)
{
	{
		UE::TWriteScopeLock Lock(SkinnedAssetLinkupCacheLock);
		SkinnedAssetLinkupCache.Remove(InSkinnedAsset);
	}
}

void USkeleton::BuildLinkupData(const USkinnedAsset* InSkinnedAsset, FSkeletonToMeshLinkup& NewMeshLinkup)
{
	const FReferenceSkeleton& SkeletonRefSkel = ReferenceSkeleton;
	const FReferenceSkeleton& MeshRefSkel = InSkinnedAsset->GetRefSkeleton();

	// First, make sure the Skeleton has all the bones the SkeletalMesh possesses.
	// This can get out of sync if a mesh was imported on that Skeleton, but the Skeleton was not saved.

	const int32 NumMeshBones = MeshRefSkel.GetNum();
	NewMeshLinkup.MeshToSkeletonTable.Empty(NumMeshBones);
	NewMeshLinkup.MeshToSkeletonTable.AddUninitialized(NumMeshBones);

#if WITH_EDITOR
	TArray<FString> MissingBoneNames;
	bool bPopulatedMissingBoneNames = false;
#endif

	for (int32 MeshBoneIndex = 0; MeshBoneIndex < NumMeshBones; MeshBoneIndex++)
	{
		const FName MeshBoneName = MeshRefSkel.GetBoneName(MeshBoneIndex);
		int32 SkeletonBoneIndex = SkeletonRefSkel.FindBoneIndex(MeshBoneName);

		if (SkeletonBoneIndex == INDEX_NONE)
		{
			bool bSpawnedErrorMessage = false;

#if WITH_EDITOR
			// If we're in editor, and skeleton is missing a bone, fix it.
			// not currently supported in-game.
			 
			// Gather remaining missing bone data
			// In case the merge succeeds, we need to gather this data before the merge to populate the message log correctly 
			if (!bPopulatedMissingBoneNames)
			{
				MissingBoneNames.Add(MeshBoneName.ToString());
				for (int32 CheckMeshBoneIndex = MeshBoneIndex + 1; CheckMeshBoneIndex < NumMeshBones; ++CheckMeshBoneIndex)
				{
					const FName MissingMeshBoneName = MeshRefSkel.GetBoneName(CheckMeshBoneIndex);
					if (SkeletonRefSkel.FindBoneIndex(MissingMeshBoneName) == INDEX_NONE)
					{
						MissingBoneNames.Add(MissingMeshBoneName.ToString());
					}
				}
				bPopulatedMissingBoneNames = true;
			}
			
			// Re-add all SkelMesh bones to the Skeleton.
			const bool bSuccessfulMerge = MergeAllBonesToBoneTree(InSkinnedAsset);

			// Fix missing bone.
			SkeletonBoneIndex = SkeletonRefSkel.FindBoneIndex(MeshBoneName);

			static FName NAME_LoadErrors("LoadErrors");

			if (IsInGameThread() || IsInSlateThread())
			{
				// If the merge failed we only display the current missing bone as this error will be hit again for future missing bones. 
				// If the merge succeeded, we don't expect this error to be hit again so we display all missing bone names. 
				const FString AllMissingBonesNames = bSuccessfulMerge ? FString::Join(MissingBoneNames, TEXT(",")) : MeshBoneName.ToString();

				FMessageLog LoadErrors(NAME_LoadErrors);

				TSharedRef<FTokenizedMessage> Message = LoadErrors.Info();
				Message->AddToken(FTextToken::Create(LOCTEXT("SkeletonBuildLinkupMissingBones1", "The Skeleton ")));
				Message->AddToken(FAssetNameToken::Create(GetPathName(), FText::FromString(GetNameSafe(this))));
				Message->AddToken(FTextToken::Create(LOCTEXT("SkeletonBuildLinkupMissingBones2", " is missing bones that SkeletalMesh ")));
				Message->AddToken(FAssetNameToken::Create(InSkinnedAsset->GetPathName(), FText::FromString(GetNameSafe(InSkinnedAsset))));
				Message->AddToken(FTextToken::Create(LOCTEXT("SkeletonBuildLinkupMissingBones3", "  needs. They will be added now. Please save the Skeleton!")));
				Message->AddToken(FTextToken::Create(FText::Format(LOCTEXT("SkeletonBuildLinkupMissingBones4", " Missing bone name(s): {0}"), FText::FromString(AllMissingBonesNames))));
				LoadErrors.Open();

				bSpawnedErrorMessage = true;
			}

#endif // WITH_EDITOR

			if (!bSpawnedErrorMessage)
			{
				// If we're not in editor, we still want to know which skeleton is missing a bone.
				UE_LOGF(LogAnimation, Error, "USkeleton::BuildLinkup: The Skeleton %ls, is missing bones that SkeletalMesh %ls needs. MeshBoneName %ls",
					*GetNameSafe(this), *GetNameSafe(InSkinnedAsset), *MeshBoneName.ToString());
			}
		}

		NewMeshLinkup.MeshToSkeletonTable[MeshBoneIndex] = SkeletonBoneIndex;
	}

	const int32 NumSkeletonBones = SkeletonRefSkel.GetNum();
	NewMeshLinkup.SkeletonToMeshTable.Empty(NumSkeletonBones);
	NewMeshLinkup.SkeletonToMeshTable.AddUninitialized(NumSkeletonBones);
	
	for (int32 SkeletonBoneIndex=0; SkeletonBoneIndex<NumSkeletonBones; SkeletonBoneIndex++)
	{
		const int32 MeshBoneIndex = MeshRefSkel.FindBoneIndex( SkeletonRefSkel.GetBoneName(SkeletonBoneIndex) );
		NewMeshLinkup.SkeletonToMeshTable[SkeletonBoneIndex] = MeshBoneIndex;
	}
}


void USkeleton::RebuildLinkup(const USkinnedAsset* InSkinnedAsset)
{
	// remove the key
	RemoveLinkup(InSkinnedAsset);
	// build new one
	AddMeshLinkupData(InSkinnedAsset);
}

void USkeleton::UpdateReferencePoseFromMesh(const USkinnedAsset* InSkinnedAsset)
{
	check(InSkinnedAsset);
	
	FReferenceSkeletonModifier RefSkelModifier(ReferenceSkeleton, this);

	for (int32 BoneIndex = 0; BoneIndex < ReferenceSkeleton.GetRawBoneNum(); BoneIndex++)
	{
		// find index from ref pose array
		const int32 MeshBoneIndex = InSkinnedAsset->GetRefSkeleton().FindRawBoneIndex(ReferenceSkeleton.GetBoneName(BoneIndex));
		if (MeshBoneIndex != INDEX_NONE)
		{
			RefSkelModifier.UpdateRefPoseTransform(BoneIndex, InSkinnedAsset->GetRefSkeleton().GetRefBonePose()[MeshBoneIndex]);
		}
	}

	MarkPackageDirty();
}

bool USkeleton::RecreateBoneTree(USkinnedAsset* InSkinnedAsset)
{
	if( InSkinnedAsset )
	{
		// regenerate Guid
		RegenerateGuid();	
		BoneTree.Empty();
		ReferenceSkeleton.Empty();

		return MergeAllBonesToBoneTree(InSkinnedAsset);
	}

	return false;
}

bool USkeleton::MergeAllBonesToBoneTree(const USkinnedAsset* InSkinnedAsset, bool bShowProgress /*= true*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeleton::MergeAllBonesToBoneTree);
	if( InSkinnedAsset )
	{
		TArray<int32> RequiredBoneIndices;

		// for now add all in this case. 
		RequiredBoneIndices.AddUninitialized(InSkinnedAsset->GetRefSkeleton().GetRawBoneNum());
		// gather bone list
		for (int32 I=0; I<InSkinnedAsset->GetRefSkeleton().GetRawBoneNum(); ++I)
		{
			RequiredBoneIndices[I] = I;
		}

		if( RequiredBoneIndices.Num() > 0 )
		{
			// merge bones to the selected skeleton
			return MergeBonesToBoneTree( InSkinnedAsset, RequiredBoneIndices, bShowProgress);
		}
	}

	return false;
}

bool USkeleton::CreateReferenceSkeletonFromMesh(const USkinnedAsset* InSkinnedAsset, const TArray<int32> & RequiredRefBones)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeleton::CreateReferenceSkeletonFromMesh);
	// Filter list, we only want bones that have their parents present in this array.
	TArray<int32> FilteredRequiredBones; 
	FAnimationRuntime::ExcludeBonesWithNoParents(RequiredRefBones, InSkinnedAsset->GetRefSkeleton(), FilteredRequiredBones);

	FReferenceSkeletonModifier RefSkelModifier(ReferenceSkeleton, this);

	if( FilteredRequiredBones.Num() > 0 )
	{
		const int32 NumBones = FilteredRequiredBones.Num();
		ReferenceSkeleton.Empty(NumBones);
		BoneTree.Empty(NumBones);
		BoneTree.AddZeroed(NumBones);

		for (int32 Index=0; Index<FilteredRequiredBones.Num(); Index++)
		{
			const int32& BoneIndex = FilteredRequiredBones[Index];

			FMeshBoneInfo NewMeshBoneInfo = InSkinnedAsset->GetRefSkeleton().GetRefBoneInfo()[BoneIndex];
			// Fix up ParentIndex for our new Skeleton.
			if( BoneIndex == 0 )
			{
				NewMeshBoneInfo.ParentIndex = INDEX_NONE; // root
			}
			else
			{
				const int32 ParentIndex = InSkinnedAsset->GetRefSkeleton().GetParentIndex(BoneIndex);
				const FName ParentName = InSkinnedAsset->GetRefSkeleton().GetBoneName(ParentIndex);
				NewMeshBoneInfo.ParentIndex = ReferenceSkeleton.FindRawBoneIndex(ParentName);
			}
			RefSkelModifier.Add(NewMeshBoneInfo, InSkinnedAsset->GetRefSkeleton().GetRefBonePose()[BoneIndex]);
		}

		return true;
	}

	return false;
}


bool USkeleton::MergeBonesToBoneTree(const USkinnedAsset* InSkinnedAsset, const TArray<int32> & RequiredRefBones, bool bShowProgress /*= true*/)
{
	// see if it needs all animation data to remap - only happens when bone structure CHANGED - added
	bool bSuccess = false;
	bool bShouldHandleHierarchyChange = false;

	// clear cache data since it won't work anymore once this is done
	ClearCacheData();

	// if it's first time
	if( BoneTree.Num() == 0 )
	{
		bSuccess = CreateReferenceSkeletonFromMesh(InSkinnedAsset, RequiredRefBones);
		bShouldHandleHierarchyChange = true;
	}
	else
	{
		// Check if we can merge in the bones
		const bool bDoParentChainCheck = !CVarAllowIncompatibleSkeletalMeshMerge.GetValueOnGameThread();
		if( IsCompatibleMesh(InSkinnedAsset, bDoParentChainCheck) )
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(USkeleton::MergeBonesToBoneTree::CompatibleBranch);
			
			const FReferenceSkeleton& InputRefSkeleton = InSkinnedAsset->GetRefSkeleton();
			const TArray<FMeshBoneInfo>& InputBoneInfos = InputRefSkeleton.GetRefBoneInfo();
			
			// Exclude bones who do not have a parent.
			TArray<int32> FilteredRequiredBones;
			FAnimationRuntime::ExcludeBonesWithNoParents(RequiredRefBones, InputRefSkeleton, FilteredRequiredBones);

			// Two modifier passes: add bones that dont already exist, then set bone parents that have changed
			{
				FReferenceSkeletonModifier RefSkelModifier(ReferenceSkeleton, this);

				// Check for bone's existence
				for (const int32 MeshBoneIndex : FilteredRequiredBones)
				{
					const FName& InputBoneName = InputRefSkeleton.GetBoneName(MeshBoneIndex);
					const int32 SkeletonBoneIndex = ReferenceSkeleton.FindRawBoneIndex(InputBoneName);
					
					// Bone doesn't already exist. Add it.
					if( SkeletonBoneIndex == INDEX_NONE )
					{
						FMeshBoneInfo NewMeshBoneInfo = InputBoneInfos[MeshBoneIndex];
						// Fix up ParentIndex for our new Skeleton.
						if( ReferenceSkeleton.GetRawBoneNum() == 0 )
						{
							NewMeshBoneInfo.ParentIndex = INDEX_NONE; // root
						}
						else
						{
							NewMeshBoneInfo.ParentIndex = ReferenceSkeleton.FindRawBoneIndex(InputRefSkeleton.GetBoneName(InputRefSkeleton.GetParentIndex(MeshBoneIndex)));
						}

						RefSkelModifier.Add(NewMeshBoneInfo, InputRefSkeleton.GetRefBonePose()[MeshBoneIndex]);
						BoneTree.AddDefaulted(1);
						bShouldHandleHierarchyChange = true;
					}
					else
					{
						// handle case-sensitive only change
						const FName& BoneName = ReferenceSkeleton.GetBoneName(SkeletonBoneIndex);
						if (InputBoneName == BoneName && !InputBoneName.IsEqual(BoneName, ENameCase::CaseSensitive))
						{
							RefSkelModifier.Rename(BoneName, InputBoneName);
						}
					}
				}
			}

			{
				FReferenceSkeletonModifier RefSkelModifier(ReferenceSkeleton, this);

				TMap<FName, FBoneNode> NameToBoneNode;
				const int32 NumBones = BoneTree.Num();
				check(NumBones == ReferenceSkeleton.GetRawRefBoneInfo().Num());

				// Check for different parents
				for (const int32 MeshBoneIndex : FilteredRequiredBones)
				{
					const FName BoneName = InputRefSkeleton.GetBoneName(MeshBoneIndex);
					const int32 SkeletonBoneIndex = ReferenceSkeleton.FindRawBoneIndex(BoneName);

					if(SkeletonBoneIndex != INDEX_NONE)
					{
						// Bone exists, check if it is in the same place in the hierarchy
						const int32 MeshParentIndex = InputRefSkeleton.GetParentIndex(MeshBoneIndex);
						const int32 SkeletonParentIndex = ReferenceSkeleton.GetRawParentIndex(SkeletonBoneIndex);
						const FName MeshParentName = MeshParentIndex != INDEX_NONE ? InputRefSkeleton.GetBoneName(MeshParentIndex) : NAME_None;
						const FName SkeletonParentName = SkeletonParentIndex != INDEX_NONE ? ReferenceSkeleton.GetRawRefBoneInfo()[SkeletonParentIndex].Name : NAME_None;

						if(MeshParentName != SkeletonParentName)
						{
							// Cache the bone tree if we are going to change the structure
							if(NameToBoneNode.Num() == 0)
							{
								NameToBoneNode.Reserve(NumBones);
								for(int32 BoneTreeIndex = 0; BoneTreeIndex < NumBones; ++BoneTreeIndex)
								{
									NameToBoneNode.Add(ReferenceSkeleton.GetRawRefBoneInfo()[BoneTreeIndex].Name, BoneTree[BoneTreeIndex]);
								}
							}

							RefSkelModifier.SetParent(BoneName, MeshParentName);
							bShouldHandleHierarchyChange = true;
						}
					}
				}

				if(NameToBoneNode.Num() > 0)
				{
					// Setting parent can re-order bones, so the skeleton's BoneTree needs to update to reflect that
					BoneTree.Reset();
					for(int32 BoneTreeIndex = 0; BoneTreeIndex < NumBones; ++BoneTreeIndex)
					{
						FName BoneName = ReferenceSkeleton.GetRawRefBoneInfo()[BoneTreeIndex].Name;
						BoneTree.Add(NameToBoneNode.FindChecked(BoneName));
					}
				}
			}

			bSuccess = true;
		}
	}

	// if succeed
	if (bShouldHandleHierarchyChange)
	{
#if WITH_EDITOR
		HandleSkeletonHierarchyChange(bShowProgress);
#endif
	}

	return bSuccess;
}

void USkeleton::SetBoneTranslationRetargetingMode(const int32 BoneIndex, EBoneTranslationRetargetingMode::Type NewRetargetingMode, bool bChildrenToo)
{
	BoneTree[BoneIndex].TranslationRetargetingMode = NewRetargetingMode;

	if( bChildrenToo )
	{
		// Bones are guaranteed to be sorted in increasing order. So children will be after this bone.
		const int32 NumBones = ReferenceSkeleton.GetRawBoneNum();
		for(int32 ChildIndex=BoneIndex+1; ChildIndex<NumBones; ChildIndex++)
		{
			if( ReferenceSkeleton.BoneIsChildOf(ChildIndex, BoneIndex) )
			{
				BoneTree[ChildIndex].TranslationRetargetingMode = NewRetargetingMode;
			}
		}
	}
}

const TArray<FTransform>& USkeleton::GetRefLocalPoses( FName RetargetSource ) const 
{
	if ( RetargetSource != NAME_None ) 
	{
		if (const FReferencePose* FoundRetargetSource = AnimRetargetSources.Find(RetargetSource))
		{
			return FoundRetargetSource->ReferencePose;
		}
	}
	return ReferenceSkeleton.GetRefBonePose();
}

#if WITH_EDITORONLY_DATA

FName USkeleton::GetRetargetSourceForMesh(USkinnedAsset* InSkinnedAsset) const
{
	FSoftObjectPath MeshPath(InSkinnedAsset);
	for(const TPair<FName, FReferencePose>& AnimRetargetSource : AnimRetargetSources)
	{
		if(AnimRetargetSource.Value.SourceReferenceMesh.ToSoftObjectPath() == MeshPath)
		{
			return AnimRetargetSource.Key;
		}
	}

	return NAME_None;
}

void USkeleton::GetRetargetSources(TArray<FName>& OutRetargetSources) const
{
	for(const TPair<FName, FReferencePose>& AnimRetargetSource : AnimRetargetSources)
	{
		OutRetargetSources.Add(AnimRetargetSource.Key);
	}
}

#endif

int32 USkeleton::GetSkeletonBoneIndexFromMeshBoneIndex(const USkinnedAsset* InSkinnedAsset, const int32 MeshBoneIndex)
{
	check(MeshBoneIndex != INDEX_NONE);
	const FSkeletonToMeshLinkup& LinkupTable = FindOrAddMeshLinkupData(InSkinnedAsset);
	return LinkupTable.MeshToSkeletonTable[MeshBoneIndex];
}

int32 USkeleton::GetMeshBoneIndexFromSkeletonBoneIndex(const USkinnedAsset* InSkinnedAsset, const int32 SkeletonBoneIndex)
{
	check(SkeletonBoneIndex != INDEX_NONE);
	const FSkeletonToMeshLinkup& LinkupTable = FindOrAddMeshLinkupData(InSkinnedAsset);
	return LinkupTable.SkeletonToMeshTable[SkeletonBoneIndex];
}


USkeletalMesh* USkeleton::GetPreviewMesh(bool bFindIfNotSet/*=false*/)
{
#if WITH_EDITORONLY_DATA
	USkeletalMesh* PreviewMesh = PreviewSkeletalMesh.LoadSynchronous();

	if(PreviewMesh && !IsCompatibleForEditor(PreviewMesh->GetSkeleton())) // fix mismatched skeleton
	{
		PreviewSkeletalMesh.Reset();
		PreviewMesh = nullptr;
	}

	// if not existing, and if bFindIfNotExisting is true, then try find one
	if(!PreviewMesh && bFindIfNotSet)
	{
		USkeletalMesh* CompatibleSkeletalMesh = FindCompatibleMesh();
		if(CompatibleSkeletalMesh)
		{
			SetPreviewMesh(CompatibleSkeletalMesh, false);
			// update PreviewMesh
			PreviewMesh = PreviewSkeletalMesh.Get();
		}
	}

	return PreviewMesh;
#else
	return nullptr;
#endif
}

USkeletalMesh* USkeleton::GetPreviewMesh() const
{
#if WITH_EDITORONLY_DATA
	if (!PreviewSkeletalMesh.IsValid())
	{
		PreviewSkeletalMesh.LoadSynchronous();
	}
	return PreviewSkeletalMesh.Get();
#else
	return nullptr;
#endif
}

void USkeleton::SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty/*=true*/)
{
#if WITH_EDITORONLY_DATA
	if (bMarkAsDirty)
	{
		Modify();
	}

	PreviewSkeletalMesh = PreviewMesh;
#endif
}

#if WITH_EDITORONLY_DATA
void USkeleton::UpdateRetargetSource( const FName Name )
{
	FReferencePose * PoseFound = AnimRetargetSources.Find(Name);

	if (PoseFound)
	{
		USkeletalMesh* ReferenceMesh;
		
		if (PoseFound->SourceReferenceMesh.IsValid())
		{
			ReferenceMesh = PoseFound->SourceReferenceMesh.Get();
		}
		else
		{
			PoseFound->SourceReferenceMesh.LoadSynchronous();
			ReferenceMesh = PoseFound->SourceReferenceMesh.Get();
		}

		if (ReferenceMesh)
		{
			FAnimationRuntime::MakeSkeletonRefPoseFromMesh(ReferenceMesh, this, PoseFound->ReferencePose);
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Reference Mesh for Retarget Source %ls has been removed.", *GetName());
		}
	}
}

void USkeleton::RefreshAllRetargetSources()
{
	for (auto Iter = AnimRetargetSources.CreateConstIterator(); Iter; ++Iter)
	{
		UpdateRetargetSource(Iter.Key());
	}
}

int32 USkeleton::GetChildBones(int32 ParentBoneIndex, TArray<int32> & Children) const
{
	return ReferenceSkeleton.GetDirectChildBones(ParentBoneIndex, Children);
}

void USkeleton::CollectAnimationNotifies(TArray<FName>& OutNotifies) const
{
	// first merge in AnimationNotifies
	if(&AnimationNotifies != &OutNotifies)
	{
		for(const FName& NotifyName : AnimationNotifies)
		{
			OutNotifies.AddUnique(NotifyName);
		}
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// @Todo : remove it when we know the asset registry is updated
	// meanwhile if you remove this, this will miss the links
	//AnimationNotifies.Empty();
	TArray<FAssetData> AssetList;
	AssetRegistryModule.Get().GetAssetsByClass(UAnimSequenceBase::StaticClass()->GetClassPathName(), AssetList, true);

	// do not clear AnimationNotifies. We can't remove old ones yet. 
	FString CurrentSkeletonName = FAssetData(this).GetExportTextName();
	for (auto Iter = AssetList.CreateConstIterator(); Iter; ++Iter)
	{
		const FAssetData& Asset = *Iter;
		const FString SkeletonValue = Asset.GetTagValueRef<FString>(TEXT("Skeleton"));
		if (SkeletonValue == CurrentSkeletonName)
		{
			FString Value;
			if (Asset.GetTagValue(USkeleton::AnimNotifyTag, Value))
			{
				TArray<FString> NotifyList;
				Value.ParseIntoArray(NotifyList, *USkeleton::AnimNotifyTagDelimiter, true);
				for (auto NotifyIter = NotifyList.CreateConstIterator(); NotifyIter; ++NotifyIter)
				{
					FString NotifyName = *NotifyIter;
					OutNotifies.AddUnique(FName(*NotifyName));
				}
			}
		}
	}
}

void USkeleton::AddNewAnimationNotify(FName NewAnimNotifyName)
{
	if (NewAnimNotifyName!=NAME_None)
	{
		AnimationNotifies.AddUnique( NewAnimNotifyName);
	}
}

void USkeleton::RemoveAnimationNotify(FName AnimNotifyName)
{
	if (AnimNotifyName != NAME_None)
	{
		AnimationNotifies.Remove(AnimNotifyName);
	}
}

void USkeleton::RenameAnimationNotify(FName OldAnimNotifyName, FName NewAnimNotifyName)
{
	if(!AnimationNotifies.Contains(NewAnimNotifyName))
	{
		for(FName& NotifyName : AnimationNotifies)
		{
			if(NotifyName == OldAnimNotifyName)
			{
				NotifyName = NewAnimNotifyName;
				break;
			}
		}
	}
}

USkeletalMesh* USkeleton::FindCompatibleMesh() const
{
	FARFilter Filter;
	Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());

	FString SkeletonString = FAssetData(this).GetExportTextName();
	
	Filter.TagsAndValues.Add(USkeletalMesh::GetSkeletonMemberName(), SkeletonString);

	TArray<FAssetData> AssetList;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssets(Filter, AssetList);

	if (AssetList.Num() > 0)
	{
		return Cast<USkeletalMesh>(AssetList[0].GetAsset());
	}

	return nullptr;
}

USkeletalMesh* USkeleton::GetAssetPreviewMesh(UObject* InAsset) 
{
	USkeletalMesh* PreviewMesh = nullptr;

	// return asset preview asset
	// if nothing is assigned, return skeleton asset
	if (UAnimationAsset* AnimAsset = Cast<UAnimationAsset>(InAsset))
	{
		PreviewMesh = AnimAsset->GetPreviewMesh();
	}
	else if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InAsset))
	{
		PreviewMesh = AnimBlueprint->GetPreviewMesh();
	}

	if (!PreviewMesh)
	{
		//The const version avoid verifying the skeleton compatibility, which can stall the thread
		const USkeleton* ThisSkeleton = this;
		PreviewMesh = ThisSkeleton->GetPreviewMesh();
		if (PreviewMesh && !PreviewMesh->IsCompiling())
		{
			//Verify the compatibility only if we are not building
			PreviewMesh = GetPreviewMesh(false);
		}
	}

	return PreviewMesh;
}

void USkeleton::LoadAdditionalPreviewSkeletalMeshes()
{
	AdditionalPreviewSkeletalMeshes.LoadSynchronous();
}

UDataAsset* USkeleton::GetAdditionalPreviewSkeletalMeshes() const
{
	return AdditionalPreviewSkeletalMeshes.Get();
}

void USkeleton::SetAdditionalPreviewSkeletalMeshes(UDataAsset* InPreviewCollectionAsset)
{
	Modify();

	AdditionalPreviewSkeletalMeshes = InPreviewCollectionAsset;
}

int32 USkeleton::ValidatePreviewAttachedObjects()
{
	int32 NumBrokenAssets = PreviewAttachedAssetContainer.ValidatePreviewAttachedObjects();

	if(NumBrokenAssets > 0)
	{
		MarkPackageDirty();
	}
	return NumBrokenAssets;
}

#if WITH_EDITOR

void USkeleton::RemoveBonesFromSkeleton( const TArray<FName>& BonesToRemove, bool bRemoveChildBones )
{
	TArray<int32> BonesRemoved = ReferenceSkeleton.RemoveBonesByName(this, BonesToRemove);
	if(BonesRemoved.Num() > 0)
	{
		BonesRemoved.Sort();
		for(int32 Index = BonesRemoved.Num()-1; Index >=0; --Index)
		{
			BoneTree.RemoveAt(BonesRemoved[Index]);
		}
		HandleSkeletonHierarchyChange();
	}
}

void USkeleton::HandleSkeletonHierarchyChange(bool bShowProgress /*= true*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeleton::HandleSkeletonHierarchyChange);
	MarkPackageDirty();

	RegenerateGuid();

	// Clear exiting MeshLinkUp tables.
	ClearCacheData();

	for (int i = VirtualBones.Num() - 1; i >= 0; --i)
	{
		FVirtualBone& VB = VirtualBones[i];

		// Note: here virtual bones can have source bound to other virtual bones
		if (ReferenceSkeleton.FindBoneIndex(VB.SourceBoneName) == INDEX_NONE ||
			ReferenceSkeleton.FindBoneIndex(VB.TargetBoneName) == INDEX_NONE)
		{
			//Virtual Bone no longer valid
			VirtualBones.RemoveAt(i);
		}
	}

	// Full rebuild of all compatible with this and with ones we are compatible with.
	UE::Anim::FSkeletonRemappingRegistry::Get().RefreshMappings(this);

	// Fix up loaded animations (any animations that aren't loaded will be fixed on load)
	int32 NumLoadedAssets = 0;
	for (TObjectIterator<UAnimationAsset> It; It; ++It)
	{
		UAnimationAsset* CurrentAnimation = *It;
		if (CurrentAnimation->GetSkeleton() == this)
		{
			NumLoadedAssets++;
		}
	}

	FScopedSlowTask SlowTask((float)NumLoadedAssets, LOCTEXT("HandleSkeletonHierarchyChange", "Rebuilding animations..."), bShowProgress);
	if (bShowProgress)
	{
		SlowTask.MakeDialog();
	}

	for (TObjectIterator<UAnimationAsset> It; It; ++It)
	{
		UAnimationAsset* CurrentAnimation = *It;
		if (CurrentAnimation->GetSkeleton() == this)
		{
			if (bShowProgress)
			{
				SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("HandleSkeletonHierarchyChange_Format", "Rebuilding Animation: {0}"), FText::FromString(CurrentAnimation->GetName())));
			}

			CurrentAnimation->ValidateSkeleton();
		}
	}

#if WITH_EDITORONLY_DATA
	RefreshAllRetargetSources();
#endif

	RefreshSkeletonMetaData();

	// Remove entries from Blend Profiles for bones that no longer exists
	for (UBlendProfile* Profile : BlendProfiles)
	{
		Profile->RefreshBoneEntriesFromName();
		Profile->CleanupBoneEntries();
	}

	OnSkeletonHierarchyChanged.Broadcast();
}

void USkeleton::HandleReferenceSkeletonChange() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeleton::HandleReferenceSkeletonChange);
	OnReferenceSkeletonChanged.Broadcast();
}

void USkeleton::RegisterOnSkeletonHierarchyChanged(const FOnSkeletonHierarchyChanged& Delegate)
{
	OnSkeletonHierarchyChanged.Add(Delegate);
}

void USkeleton::UnregisterOnSkeletonHierarchyChanged(FDelegateUserObject Unregister)
{
	OnSkeletonHierarchyChanged.RemoveAll(Unregister);
}

void USkeleton::RegisterOnReferenceSkeletonChanged(const FOnReferenceSkeletonChanged& Delegate)
{
	OnReferenceSkeletonChanged.Add(Delegate);
}

void USkeleton::UnregisterOnReferenceSkeletonChanged(FDelegateUserObject Unregister)
{
	OnReferenceSkeletonChanged.RemoveAll(Unregister);
}

#endif

#endif // WITH_EDITORONLY_DATA

const TArray<FAnimSlotGroup>& USkeleton::GetSlotGroups() const
{
	return SlotGroups;
}

void USkeleton::BuildSlotToGroupMap(bool bInRemoveDuplicates)
{
	SlotToGroupNameMap.Empty();

	for (FAnimSlotGroup& SlotGroup : SlotGroups)
	{
		for (const FName& SlotName : SlotGroup.SlotNames)
		{
			SlotToGroupNameMap.Add(SlotName, SlotGroup.GroupName);
		}
	}

	// Use the map we've just build to rebuild the slot groups
	if(bInRemoveDuplicates)
	{
		for(FAnimSlotGroup& SlotGroup : SlotGroups)
		{
			SlotGroup.SlotNames.Empty(SlotGroup.SlotNames.Num());

			for(TPair<FName, FName>& SlotToGroupPair : SlotToGroupNameMap)
			{
				if(SlotToGroupPair.Value == SlotGroup.GroupName)
				{
					SlotGroup.SlotNames.Add(SlotToGroupPair.Key);
				}
			}
		}

	}
}

FAnimSlotGroup* USkeleton::FindAnimSlotGroup(const FName& InGroupName)
{
	return SlotGroups.FindByPredicate([&InGroupName](const FAnimSlotGroup& Item)
	{
		return Item.GroupName == InGroupName;
	});
}

const FAnimSlotGroup* USkeleton::FindAnimSlotGroup(const FName& InGroupName) const
{
	return SlotGroups.FindByPredicate([&InGroupName](const FAnimSlotGroup& Item)
	{
		return Item.GroupName == InGroupName;
	});
}

bool USkeleton::ContainsSlotName(const FName& InSlotName) const
{
	return SlotToGroupNameMap.Contains(InSlotName);
}

bool USkeleton::RegisterSlotNode(const FName& InSlotName)
{
	// verify the slot name exists, if not create it in the default group.
	if (!ContainsSlotName(InSlotName))
	{
		SetSlotGroupName(InSlotName, FAnimSlotGroup::DefaultGroupName);
		return true;
	}

	return false;
}

void USkeleton::SetSlotGroupName(const FName& InSlotName, const FName& InGroupName)
{
// See if Slot already exists and belongs to a group.
	const FName* FoundGroupNamePtr = SlotToGroupNameMap.Find(InSlotName);

	// If slot exists, but is not in the right group, remove it from there
	if (FoundGroupNamePtr && ((*FoundGroupNamePtr) != InGroupName))
	{
		FAnimSlotGroup* OldSlotGroupPtr = FindAnimSlotGroup(*FoundGroupNamePtr);
		if (OldSlotGroupPtr)
		{
			OldSlotGroupPtr->SlotNames.RemoveSingleSwap(InSlotName);
		}
	}

	// Add the slot to the right group if it's not
	if ((FoundGroupNamePtr == NULL) || (*FoundGroupNamePtr != InGroupName))
	{
		// If the SlotGroup does not exist, create it.
		FAnimSlotGroup* SlotGroupPtr = FindAnimSlotGroup(InGroupName);
		if (SlotGroupPtr == NULL)
		{
			SlotGroups.AddZeroed(1);
			SlotGroupPtr = &SlotGroups.Last();
			SlotGroupPtr->GroupName = InGroupName;
		}
		// Add Slot to group.
		SlotGroupPtr->SlotNames.Add(InSlotName);
		// Keep our TMap up to date.
		SlotToGroupNameMap.Add(InSlotName, InGroupName);
	}
}

bool USkeleton::AddSlotGroupName(const FName& InNewGroupName)
{
	FAnimSlotGroup* ExistingSlotGroupPtr = FindAnimSlotGroup(InNewGroupName);
	if (ExistingSlotGroupPtr == NULL)
	{
		// if not found, create a new one.
		SlotGroups.AddZeroed(1);
		ExistingSlotGroupPtr = &SlotGroups.Last();
		ExistingSlotGroupPtr->GroupName = InNewGroupName;
		return true;
	}

	return false;
}

FName USkeleton::GetSlotGroupName(const FName& InSlotName) const
{
	const FName* FoundGroupNamePtr = SlotToGroupNameMap.Find(InSlotName);
	if (FoundGroupNamePtr)
	{
		return *FoundGroupNamePtr;
	}

	// If Group name cannot be found, use DefaultSlotGroupName.
	return FAnimSlotGroup::DefaultGroupName;
}

void USkeleton::RemoveSlotName(const FName& InSlotName)
{
	FName GroupName = GetSlotGroupName(InSlotName);
	
	if(SlotToGroupNameMap.Remove(InSlotName) > 0)
	{
		FAnimSlotGroup* SlotGroup = FindAnimSlotGroup(GroupName);
		SlotGroup->SlotNames.Remove(InSlotName);
	}
}

void USkeleton::RemoveSlotGroup(const FName& InSlotGroupName)
{
	FAnimSlotGroup* SlotGroup = FindAnimSlotGroup(InSlotGroupName);
	// Remove slot mappings
	for(const FName& SlotName : SlotGroup->SlotNames)
	{
		SlotToGroupNameMap.Remove(SlotName);
	}

	// Remove group
	SlotGroups.RemoveAll([&InSlotGroupName](const FAnimSlotGroup& Item)
	{
		return Item.GroupName == InSlotGroupName;
	});
}

void USkeleton::RenameSlotName(const FName& OldName, const FName& NewName)
{
	// Can't rename a name that doesn't exist
	check(ContainsSlotName(OldName))

	FName GroupName = GetSlotGroupName(OldName);
	RemoveSlotName(OldName);
	SetSlotGroupName(NewName, GroupName);
}

bool USkeleton::AddCurveMetaData(FName CurveName, bool bTransact)
{
	UAnimCurveMetaData* AnimCurveMetaData = GetOrCreateCurveMetaDataObject();
	return AnimCurveMetaData->AddCurveMetaData(CurveName, bTransact);
}

#if WITH_EDITOR

bool USkeleton::RenameCurveMetaData(FName OldName, FName NewName)
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->RenameCurveMetaData(OldName, NewName);
	}
	return false;
}

bool USkeleton::RemoveCurveMetaData(FName CurveName)
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->RemoveCurveMetaData(CurveName);
	}
	return false;
}

bool USkeleton::RemoveCurveMetaData(TArrayView<FName> CurveNames)
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->RemoveCurveMetaData(CurveNames);
	}
	return false;
}

bool USkeleton::GetCurveMetaDataMaterial(FName CurveName) const
{
	if (const UAnimCurveMetaData* AnimCurveMetaData = const_cast<USkeleton*>(this)->GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->GetCurveMetaDataMaterial(CurveName);
	}

	return false;
}

bool USkeleton::GetCurveMetaDataMorphTarget(FName CurveName) const
{
	if (const UAnimCurveMetaData* AnimCurveMetaData = const_cast<USkeleton*>(this)->GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->GetCurveMetaDataMorphTarget(CurveName);
	}

	return false;
}

void USkeleton::SetCurveMetaDataMaterial(FName CurveName, bool bOverrideMaterial)
{
	if (UAnimCurveMetaData* AnimCurveMetaData = GetAssetUserData<UAnimCurveMetaData>())
	{
		AnimCurveMetaData->SetCurveMetaDataMaterial(CurveName, bOverrideMaterial);
	}
}

void USkeleton::SetCurveMetaDataMorphTarget(FName CurveName, bool bOverrideMorphTarget)
{
	if (UAnimCurveMetaData* AnimCurveMetaData = GetAssetUserData<UAnimCurveMetaData>())
	{
		AnimCurveMetaData->SetCurveMetaDataMorphTarget(CurveName, bOverrideMorphTarget);
	}
}

#endif // WITH_EDITOR

uint16 USkeleton::GetAnimCurveUidVersion() const
{
	if(UAnimCurveMetaData* AnimCurveMetaData = const_cast<USkeleton*>(this)->GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->GetVersionNumber();
	}
	return 0;
}

void USkeleton::GetCurveMetaDataNames(TArray<FName>& OutNames) const
{
	if(const UAnimCurveMetaData* AnimCurveMetaData = const_cast<USkeleton*>(this)->GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->GetCurveMetaDataNames(OutNames);
	}
}

void USkeleton::RegenerateGuid()
{
	Guid = FGuid::NewGuid();
	check(Guid.IsValid());
}

void USkeleton::RegenerateVirtualBoneGuid()
{
	VirtualBoneGuid = FGuid::NewGuid();
	check(VirtualBoneGuid.IsValid());
}

FCurveMetaData* USkeleton::GetCurveMetaData(FName CurveName)
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->GetCurveMetaData(CurveName);
	}
	return nullptr;
}

const FCurveMetaData* USkeleton::GetCurveMetaData(FName CurveName) const
{
	if(const UAnimCurveMetaData* AnimCurveMetaData = const_cast<USkeleton*>(this)->GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->GetCurveMetaData(CurveName);
	}
	return nullptr;
}

void USkeleton::ForEachCurveMetaData(TFunctionRef<void(FName, const FCurveMetaData&)> InFunction) const
{
	if(const UAnimCurveMetaData* AnimCurveMetaData = const_cast<USkeleton*>(this)->GetAssetUserData<UAnimCurveMetaData>())
	{
		AnimCurveMetaData->ForEachCurveMetaData(InFunction);
	}
}

int32 USkeleton::GetNumCurveMetaData() const
{
	if(const UAnimCurveMetaData* AnimCurveMetaData = const_cast<USkeleton*>(this)->GetAssetUserData<UAnimCurveMetaData>())
	{
		return AnimCurveMetaData->GetNumCurveMetaData();
	}
	return 0;
}

void USkeleton::AccumulateCurveMetaData(FName CurveName, bool bMaterialSet, bool bMorphtargetSet)
{
	if (bMaterialSet || bMorphtargetSet)
	{
		// Add curve if not already present
		AddCurveMetaData(CurveName);

		FCurveMetaData* FoundCurveMetaData = GetCurveMetaData(CurveName);
		check(FoundCurveMetaData);

		bool bOldMaterial = FoundCurveMetaData->Type.bMaterial;
		bool bOldMorphtarget = FoundCurveMetaData->Type.bMorphtarget;
		// we don't want to undo previous flags, if it was true, we just allow more to it. 
		FoundCurveMetaData->Type.bMaterial |= bMaterialSet;
		FoundCurveMetaData->Type.bMorphtarget |= bMorphtargetSet;

		if (bOldMaterial != FoundCurveMetaData->Type.bMaterial 
			|| bOldMorphtarget != FoundCurveMetaData->Type.bMorphtarget)
		{
			MarkPackageDirty();
		}
	}
}

bool USkeleton::AddNewVirtualBone(const FName SourceBoneName, const FName TargetBoneName)
{
	FName Dummy;
	return AddNewVirtualBone(SourceBoneName, TargetBoneName, Dummy);
}

bool USkeleton::AddNewVirtualBone(const FName SourceBoneName, const FName TargetBoneName, FName& NewVirtualBoneName)
{
	for (const FVirtualBone& SSBone : VirtualBones)
	{
		if (SSBone.SourceBoneName == SourceBoneName &&
			SSBone.TargetBoneName == TargetBoneName)
		{
			return false;
		}
	}
	Modify();
	VirtualBones.Add(FVirtualBone(SourceBoneName, TargetBoneName));
	NewVirtualBoneName = VirtualBones.Last().VirtualBoneName;

	RegenerateVirtualBoneGuid();
	HandleVirtualBoneChanges();


	return true;
}

bool USkeleton::AddNewNamedVirtualBone(const FName SourceBoneName, const FName TargetBoneName, const FName VirtualBoneName)
{
	if (!VirtualBoneNameHelpers::CheckVirtualBonePrefix(VirtualBoneName.ToString()))
	{
		return false;
	}

	for (const FVirtualBone& SSBone : VirtualBones)
	{
		if ((SSBone.SourceBoneName == SourceBoneName && SSBone.TargetBoneName == TargetBoneName) ||
			SSBone.VirtualBoneName == VirtualBoneName)
		{
			return false;
		}
	}

	Modify();

	VirtualBones.Emplace(SourceBoneName, TargetBoneName, VirtualBoneName);

	RegenerateVirtualBoneGuid();
	HandleVirtualBoneChanges();

	return true;
}

int32 FindBoneByName(const FName& BoneName, TArray<FVirtualBone>& Bones)
{
	for (int32 Idx = 0; Idx < Bones.Num(); ++Idx)
	{
		if (Bones[Idx].VirtualBoneName == BoneName)
		{
			return Idx;
		}
	}
	return INDEX_NONE;
}

void USkeleton::RemoveVirtualBones(const TArray<FName>& BonesToRemove)
{
	Modify();
	for (const FName& BoneName : BonesToRemove)
	{
		int32 Idx = FindBoneByName(BoneName, VirtualBones);
		if (Idx != INDEX_NONE)
		{
			FName Parent = VirtualBones[Idx].SourceBoneName;
			for (FVirtualBone& VB : VirtualBones)
			{
				if (VB.SourceBoneName == BoneName)
				{
					VB.SourceBoneName = Parent;
				}
			}
			VirtualBones.RemoveAt(Idx,EAllowShrinking::No);

			// @todo: This might be a slow operation if there's a large amount of blend profiles and entries
			int32 BoneIdx = GetReferenceSkeleton().FindBoneIndex(BoneName);
			if(BoneIdx != INDEX_NONE)
			{
				for (UBlendProfile* Profile : BlendProfiles)
				{
					Profile->RemoveEntry(BoneIdx);
				}
			}
		}
	}

	RegenerateVirtualBoneGuid();
	HandleVirtualBoneChanges();

	// Blend profiles cache bone names and indices, make sure they remain in sync when the indices change
	for (UBlendProfile* Profile : BlendProfiles)
	{
		Profile->RefreshBoneEntriesFromName();
	}
}

void USkeleton::RenameVirtualBone(const FName OriginalBoneName, const FName NewBoneName)
{
	bool bModified = false;

	for (FVirtualBone& VB : VirtualBones)
	{
		if (VB.VirtualBoneName == OriginalBoneName)
		{
			if (!bModified)
			{
				bModified = true;
				Modify();
			}

			VB.VirtualBoneName = NewBoneName;
		}

		if (VB.SourceBoneName == OriginalBoneName)
		{
			if (!bModified)
			{
				bModified = true;
				Modify();
			}
			VB.SourceBoneName = NewBoneName;
		}
	}

	if (bModified)
	{
		RegenerateVirtualBoneGuid();
		HandleVirtualBoneChanges();

		// @todo: This might be a slow operation if there's a large amount of blend profiles and entries
		int32 BoneIdx = GetReferenceSkeleton().FindBoneIndex(NewBoneName);
		if (BoneIdx != INDEX_NONE)
		{
			for (UBlendProfile* Profile : BlendProfiles)
			{
				Profile->RefreshBoneEntry(BoneIdx);
			}
		}
	}
}

void USkeleton::HandleVirtualBoneChanges()
{
	constexpr bool bRebuildNameMap = false;
	ReferenceSkeleton.RebuildRefSkeleton(this, bRebuildNameMap);

	// store skeletal meshes that are also transacting to avoid re-registering the component here
	// as it will be done later in USkeletalMesh::PostEditUndo()
	TArray<USkeletalMesh*> SkeletalMeshTransacting;
	
	for (TObjectIterator<USkeletalMesh> ItMesh; ItMesh; ++ItMesh)
	{
		USkeletalMesh* SkelMesh = *ItMesh;
		if (SkelMesh->GetSkeleton() == this)
		{
			// also have to update retarget base pose
			SkelMesh->GetRefSkeleton().RebuildRefSkeleton(this, bRebuildNameMap);
			RebuildLinkup(SkelMesh);

#if WITH_EDITOR
			if (SkelMesh->IsTransacting())
			{
				SkeletalMeshTransacting.Add(SkelMesh);
			}
#endif
		}
	}

	// refresh curve meta data that contains joint info
	RefreshSkeletonMetaData();

	auto NeedsReRegistration = [this, &SkeletalMeshTransacting](const ISkinnedMeshComponent* InMeshComponent)
	{
		if (!InMeshComponent)
		{
			return false;
		}

		USkinnedAsset* SkinnedAsset = InMeshComponent->GetSkinnedAsset();
		if (!SkinnedAsset || SkinnedAsset->GetSkeleton() != this)
		{
			return false;
		}

#if WITH_EDITOR
		return !SkeletalMeshTransacting.Contains(SkinnedAsset);
#else
		return true;
#endif
	};

#if WITH_EDITOR
	FObjectCacheContextScope ObjectCacheScope;
	for (ISkinnedMeshComponent* Component : ObjectCacheScope.GetContext().GetSkinnedMeshComponents())
	{
		if (NeedsReRegistration(Component))
		{
			Component->PostSkeletalHierarchyChange();
		}
	}

	OnSkeletonHierarchyChanged.Broadcast();
#endif
}

#if WITH_EDITOR
void USkeleton::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	TStringBuilder<256> CompatibleSkeletonsBuilder;
	
	for(const TSoftObjectPtr<USkeleton>& CompatibleSkeleton : CompatibleSkeletons)
	{
		const FString ExportPath = FObjectPropertyBase::GetExportPath(USkeleton::StaticClass()->GetClassPathName(), CompatibleSkeleton.ToSoftObjectPath().GetAssetPathString());
		CompatibleSkeletonsBuilder.Append(ExportPath);
		CompatibleSkeletonsBuilder.Append(USkeleton::CompatibleSkeletonsTagDelimiter);
	}

	Context.AddTag(FAssetRegistryTag(USkeleton::CompatibleSkeletonsNameTag, CompatibleSkeletonsBuilder.ToString(), FAssetRegistryTag::TT_Hidden));

	// Output sync notify names we use
	TStringBuilder<256> NotifiesBuilder;
	NotifiesBuilder.Append(USkeleton::AnimNotifyTagDelimiter);

	for(FName NotifyName : AnimationNotifies)
	{
		NotifiesBuilder.Append(NotifyName.ToString());
		NotifiesBuilder.Append(USkeleton::AnimNotifyTagDelimiter);
	}

	Context.AddTag(FAssetRegistryTag(USkeleton::AnimNotifyTag, NotifiesBuilder.ToString(), FAssetRegistryTag::TT_Hidden));
	
	// Output sync marker names we use
	TStringBuilder<256> SyncMarkersBuilder;
	SyncMarkersBuilder.Append(USkeleton::AnimSyncMarkerTagDelimiter);

	for(FName SyncMarker : ExistingMarkerNames)
	{
		SyncMarkersBuilder.Append(SyncMarker.ToString());
		SyncMarkersBuilder.Append(USkeleton::AnimSyncMarkerTagDelimiter);
	}

	Context.AddTag(FAssetRegistryTag(USkeleton::AnimSyncMarkerTag, SyncMarkersBuilder.ToString(), FAssetRegistryTag::TT_Hidden));
	
	// Allow asset user data to output tags
	for(const UAssetUserData* AssetUserDataItem : *GetAssetUserDataArray())
	{
		if (AssetUserDataItem)
		{
			AssetUserDataItem->GetAssetRegistryTags(Context);
		}
	}
}

#endif //WITH_EDITOR

UBlendProfile* USkeleton::GetBlendProfile(const FName& InProfileName)
{
	for (const TObjectPtr<UBlendProfile>& Profile : BlendProfiles)
	{
		if (Profile->GetFName() == InProfileName)
		{
			return Profile;
		}
	}
	return nullptr;
}

UBlendProfile* USkeleton::CreateNewBlendProfile(const FName& InProfileName)
{
	Modify();
	UBlendProfile* NewProfile = NewObject<UBlendProfile>(this, InProfileName, RF_Public | RF_Transactional);
	BlendProfiles.Add(NewProfile);

	return NewProfile;
}

UBlendProfile* USkeleton::RenameBlendProfile(const FName& InProfileName, const FName& InNewProfileName)
{
	if (!GetBlendProfile(InNewProfileName)) // we can not rename if the new name already exists
	{
		UBlendProfile* BlendProfile = GetBlendProfile(InProfileName);
		if (BlendProfile)
		{
			Modify();
			if (BlendProfile->Rename(*InNewProfileName.ToString(), this, REN_DontCreateRedirectors))
			{
				return BlendProfile;
			}
		}
	}

	return nullptr;
}

USkeletalMeshSocket* USkeleton::FindSocket(FName InSocketName) const
{
	int32 DummyIndex;
	return FindSocketAndIndex(InSocketName, DummyIndex);
}

USkeletalMeshSocket* USkeleton::FindSocketAndIndex(FName InSocketName, int32& OutIndex) const
{
	OutIndex = INDEX_NONE;
	if (InSocketName == NAME_None)
	{
		return nullptr;
	}

	for (int32 i = 0; i < Sockets.Num(); ++i)
	{
		USkeletalMeshSocket* Socket = Sockets[i];
		if (Socket && Socket->SocketName == InSocketName)
		{
			OutIndex = i;
			return Socket;
		}
	}

	return nullptr;
}


void USkeleton::AddAssetUserData( UAssetUserData* InUserData)
{
	if (InUserData != NULL)
	{
		RemoveUserDataOfClass(InUserData->GetClass());
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* USkeleton::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	const TArray<UAssetUserData*>* ArrayPtr = GetAssetUserDataArray();
	for (int32 DataIdx = 0; DataIdx < ArrayPtr->Num(); DataIdx++)
	{
		UAssetUserData* Datum = (*ArrayPtr)[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

void USkeleton::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}

#if WITH_EDITOR
	for (int32 DataIdx = 0; DataIdx < AssetUserDataEditorOnly.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserDataEditorOnly[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserDataEditorOnly.RemoveAt(DataIdx);
			return;
		}
	}
#endif
}

const TArray<UAssetUserData*>* USkeleton::GetAssetUserDataArray() const
{
#if WITH_EDITOR
	if (IsRunningCookCommandlet())
	{
		return &ToRawPtrTArrayUnsafe(AssetUserData);
	}
	else
	{
		static thread_local TArray<TObjectPtr<UAssetUserData>> CachedAssetUserData;
		CachedAssetUserData.Reset();
		CachedAssetUserData.Append(AssetUserData);
		CachedAssetUserData.Append(AssetUserDataEditorOnly);
		return &ToRawPtrTArrayUnsafe(CachedAssetUserData);
	}
#else
	return &ToRawPtrTArrayUnsafe(AssetUserData);
#endif
}

void USkeleton::HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)
	{
		for (const auto& RepointedObjectPair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (USkeleton* NewObject = Cast<USkeleton>(RepointedObjectPair.Value))
			{
				NewObject->HandleVirtualBoneChanges(); // Reloading Skeletons can invalidate virtual bones so refresh
			}
		}
	}
}

void USkeleton::RefreshSkeletonMetaData()
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAssetUserData<UAnimCurveMetaData>())
	{
		AnimCurveMetaData->RefreshBoneIndices(this);
	}
}

UAnimCurveMetaData* USkeleton::GetOrCreateCurveMetaDataObject()
{
	UAnimCurveMetaData* AnimCurveMetaData = GetAssetUserData<UAnimCurveMetaData>();
	if (AnimCurveMetaData == nullptr)
	{
		AnimCurveMetaData = NewObject<UAnimCurveMetaData>(this, NAME_None, RF_Transactional);
		AddAssetUserData(AnimCurveMetaData);
	}

	return AnimCurveMetaData;
}

bool USkeleton::GetUseRetargetModesFromCompatibleSkeleton() const
{
	return bUseRetargetModesFromCompatibleSkeleton;
}

void USkeleton::SetUseRetargetModesFromCompatibleSkeleton(bool bUse)
{
	bUseRetargetModesFromCompatibleSkeleton = bUse;
}

#undef LOCTEXT_NAMESPACE 

