// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/SkeletonRemapping.h"

#include "AnimationRuntime.h"
#include "Animation/Skeleton.h"

FSkeletonRemapping::FSkeletonRemapping(const USkeleton* InSourceSkeleton, const USkeleton* InTargetSkeleton)
	: SourceSkeleton(InSourceSkeleton)
	, TargetSkeleton(InTargetSkeleton)
{
	GenerateMapping();
}

void FSkeletonRemapping::RegenerateMapping()
{
	SourceToTargetBoneIndexes.Reset();
	TargetToSourceBoneIndexes.Reset();
	RetargetingTable.Reset();

	GenerateMapping();
}

void FSkeletonRemapping::GenerateMapping()
{
	if (!GetSourceSkeleton().IsValid() || !GetTargetSkeleton().IsValid())
	{
		return;
	}
	
	const FReferenceSkeleton& SourceReferenceSkeleton = SourceSkeleton->GetReferenceSkeleton();
	const FReferenceSkeleton& TargetReferenceSkeleton = TargetSkeleton->GetReferenceSkeleton();

	const int32 SourceNumBones = SourceReferenceSkeleton.GetNum();
	const int32 TargetNumBones = TargetReferenceSkeleton.GetNum();

	SourceToTargetBoneIndexes.SetNumUninitialized(SourceNumBones);
	TargetToSourceBoneIndexes.SetNumUninitialized(TargetNumBones);
	RetargetingTable.SetNumUninitialized(TargetNumBones);

	TArrayView<const FTransform> SourceLocalTransforms;
	TArrayView<const FTransform> TargetLocalTransforms;
/*
	// Build the mapping from source to target bones through the remapping rig if one exists
	if (true)	// TODO_SKELETON_REMAPPING: Use a remapping rig if it exists
	{
		for (int32 SourceBoneIndex = 0; SourceBoneIndex < SourceNumBones; ++SourceBoneIndex)
		{
			const FName BoneName = SourceReferenceSkeleton.GetBoneName(SourceBoneIndex);
			int32 TargetBoneIndex = TargetReferenceSkeleton.FindBoneIndex(BoneName);
			SourceToTargetBoneIndexes[SourceBoneIndex] = TargetBoneIndex;
		}

		// Get the matched source and target rest poses from the remapping rig
		//
		// TODO_SKELETON_REMAPPING: For now we'll just use the skeleton's rest poses, but we should really be pulling these
		// poses from a remapping rig so that the use can better align the poses.  Note that we'll want the remapping rig
		// to be independent from the USkeleton itself because a skeleton may want to participate in multiple remappings with
		// different other skeletons and may have to pose itself differently to align with the different other skeletons
		//
		SourceLocalTransforms = MakeArrayView(SourceReferenceSkeleton.GetRefBonePose());
		TargetLocalTransforms = MakeArrayView(TargetReferenceSkeleton.GetRefBonePose());
	}
	else // Fall back to simple name matching if there is no rig
*/
	{
		// Match source to target bones by name lookup between source and target skeletons
		for (int32 SourceBoneIndex = 0; SourceBoneIndex < SourceNumBones; ++SourceBoneIndex)
		{
			const FName BoneName = SourceReferenceSkeleton.GetBoneName(SourceBoneIndex);
			const int32 TargetBoneIndex = TargetReferenceSkeleton.FindBoneIndex(BoneName);
			SourceToTargetBoneIndexes[SourceBoneIndex] = TargetBoneIndex;
		}

		// Get the matched (hopefully) source and target rest poses from the source and target skeletons
		SourceLocalTransforms = MakeArrayView(SourceReferenceSkeleton.GetRefBonePose());
		TargetLocalTransforms = MakeArrayView(TargetReferenceSkeleton.GetRefBonePose());
	}

	// Force the roots to map onto each other regardless of their names
	SourceToTargetBoneIndexes[0] = 0;

	// Build the reverse mapping from target back to source bones
	FMemory::Memset(TargetToSourceBoneIndexes.GetData(), static_cast<uint8>(INDEX_NONE), TargetNumBones * TargetToSourceBoneIndexes.GetTypeSize());
	for (int32 SourceBoneIndex = 0; SourceBoneIndex < SourceNumBones; ++SourceBoneIndex)
	{
		const int32 TargetBoneIndex = SourceToTargetBoneIndexes[SourceBoneIndex];
		if (TargetBoneIndex != INDEX_NONE)
		{
			TargetToSourceBoneIndexes[TargetBoneIndex] = SourceBoneIndex;
		}
	}

	TArray<FTransform> SourceComponentTransforms;
	TArray<FTransform> TargetComponentTransforms;
	FAnimationRuntime::FillUpComponentSpaceTransforms(SourceReferenceSkeleton, SourceLocalTransforms, SourceComponentTransforms);
	FAnimationRuntime::FillUpComponentSpaceTransforms(TargetReferenceSkeleton, TargetLocalTransforms, TargetComponentTransforms);

	// Calculate the retargeting constants to map from source skeleton space to target skeleton space
	//
	// Simply remapping joint indices is usually not sufficient to give us the desired result pose if the source and target
	// skeletons are built with different conventions for joint orientations.  We therefore need to compute a remapping
	// between the source and target joint orientations, which we do in terms of delta rotations from the rest pose:
	//
	//		Q = P * D * R
	//
	// where:
	//
	//		Q is the final joint orientation in component space
	//		P is the parent joint orientation in component space
	//		D is the delta rotation that we want to remap
	//		R is the local space rest pose orientation for the joint
	//
	// We want to find a mapping from the source to the target:
	//
	//		Ps * Ds * Rs  -->  Pt * Dt * Rt
	//
	// such that the deltas produce equivalent rotations on the mesh even if their parent rotation frames are different.
	// In other words, we need to find Dt such that its component space rotation is equivalent to Ds.  To convert a rotation
	// from local space (D) to component space (C), given its parent (P), we have:
	//
	//		C * P = P * D
	//		C = P * D * P⁻¹		(this uses the sandwich product to rotate the rotation axis from local to mesh space)
	//
	// Setting the source and target component space rotations to be equal (Cs = Ct) then gives us:
	//
	//		Ps * Ds * Ps⁻¹ = Pt * Dt * Pt⁻¹
	//
	// which we then solve for Dt to get:
	//
	//		Dt = Pt⁻¹ * Ps * Ds * Ps⁻¹ * Pt
	//
	// However, when we're remapping an animation pose, we will have the local transforms rather than the deltas from the
	// rest pose, so we also need to convert between these local transforms and the equivalent deltas:
	//
	//		L = D * R
	//		D = L * R⁻¹
	//
	// Combining that with our equation for Dt, we get:
	//
	//		Lt * Rt⁻¹ = Pt⁻¹ * Ps * Ls * Rs⁻¹ * Ps⁻¹ * Pt
	//
	// Solving for Lt then gives us:
	//
	//		Lt = Pt⁻¹ * Ps * Ls * Rs⁻¹ * Ps⁻¹ * Pt * Rt
	//
	// Finally, factoring out the constant terms (which we pre-compute here) gives us:
	//
	//		Lt = Q0 * Ls * Q1
	//		Q0 = Pt⁻¹ * Ps
	//		Q1 = Rs⁻¹ * Ps⁻¹ * Pt * Rt
	//
	// Note that when remapping additive animations, we drop the rest pose terms, but we still need to convert between the
	// source and target rotation frames. The terms drop because an additive rotation needs to be applied to a base one to
	// become a local space rotation. To that end, we can use any base as we are interested in the delta between source/target.
	// Using the bind pose as our base quickly cancels out the terms.
	// Dropping Rs and Rt from the equations above gives us:
	//
	//		Lt = Pt⁻¹ * Ps * Ls * Ps⁻¹ * Pt			(for additive animations)
	//
	// which is equivalent to the following in terms of our precomputed constants:
	//
	//		Lt = Q0 * Ls * Q0⁻¹						(for additive animations)
	//
	// For mesh space additive animations, the source rotations already represent a mesh space delta and as such no fix-up needs
	// to applied for those.

	RetargetingTable[0] = MakeTuple(FQuat::Identity, SourceLocalTransforms[0].GetRotation().Inverse() * TargetLocalTransforms[0].GetRotation());
	for (int32 TargetBoneIndex = 1; TargetBoneIndex < TargetNumBones; ++TargetBoneIndex)
	{
		const int32 SourceBoneIndex = TargetToSourceBoneIndexes[TargetBoneIndex];
		if (SourceBoneIndex != INDEX_NONE)
		{
			const int32 SourceParentIndex = SourceReferenceSkeleton.GetParentIndex(SourceBoneIndex);
			const int32 TargetParentIndex = TargetReferenceSkeleton.GetParentIndex(TargetBoneIndex);
			check(SourceParentIndex != INDEX_NONE);
			check(TargetParentIndex != INDEX_NONE);

			const FQuat PS = SourceComponentTransforms[SourceParentIndex].GetRotation();
			const FQuat PT = TargetComponentTransforms[TargetParentIndex].GetRotation();

			const FQuat RS = SourceLocalTransforms[SourceBoneIndex].GetRotation();
			const FQuat RT = TargetLocalTransforms[TargetBoneIndex].GetRotation();

			const FQuat Q0 = PT.Inverse() * PS;
			const FQuat Q1 = RS.Inverse() * PS.Inverse() * PT * RT;

			RetargetingTable[TargetBoneIndex] = MakeTuple(Q0, Q1);
		}
		else
		{
			RetargetingTable[TargetBoneIndex] = MakeTuple(FQuat::Identity, FQuat::Identity);
		}
	}
}

void FSkeletonRemapping::ComposeWith(const FSkeletonRemapping& OtherSkeletonRemapping)
{
	check(OtherSkeletonRemapping.SourceSkeleton == TargetSkeleton);

	TargetSkeleton = OtherSkeletonRemapping.TargetSkeleton;

	const int32 SourceNumBones = SourceToTargetBoneIndexes.Num();
	const int32 TargetNumBones = OtherSkeletonRemapping.TargetToSourceBoneIndexes.Num();

	TArray<int32> NewTargetToSourceBoneIndexes;
	TArray<TTuple<FQuat, FQuat>> NewRetargetingTable;

	NewTargetToSourceBoneIndexes.SetNumUninitialized(TargetNumBones);
	NewRetargetingTable.SetNumUninitialized(TargetNumBones);

	// Compose the retargeting constants
	for (int32 NewTargetBoneIndex = 0; NewTargetBoneIndex < TargetNumBones; ++NewTargetBoneIndex)
	{
		const int32 OldTargetBoneIndex = OtherSkeletonRemapping.TargetToSourceBoneIndexes[NewTargetBoneIndex];
		const int32 OldSourceBoneIndex = (OldTargetBoneIndex != INDEX_NONE) ? TargetToSourceBoneIndexes[OldTargetBoneIndex] : INDEX_NONE;

		if (OldSourceBoneIndex != INDEX_NONE)
		{
			const TTuple<FQuat, FQuat>& OldQQ = RetargetingTable[OldTargetBoneIndex];
			const TTuple<FQuat, FQuat>& NewQQ = OtherSkeletonRemapping.RetargetingTable[NewTargetBoneIndex];
			const FQuat Q0 = NewQQ.Get<0>() * OldQQ.Get<0>();
			const FQuat Q1 = OldQQ.Get<1>() * NewQQ.Get<1>();

			NewTargetToSourceBoneIndexes[NewTargetBoneIndex] = OldSourceBoneIndex;
			NewRetargetingTable[NewTargetBoneIndex] = MakeTuple(Q0, Q1);
		}
		else
		{
			NewTargetToSourceBoneIndexes[NewTargetBoneIndex] = INDEX_NONE;
			NewRetargetingTable[NewTargetBoneIndex] = MakeTuple(FQuat::Identity, FQuat::Identity);
		}
	}

	TargetToSourceBoneIndexes = MoveTemp(NewTargetToSourceBoneIndexes);
	RetargetingTable = MoveTemp(NewRetargetingTable);

	// Rebuild the mapping from source bones to the target bones
	FMemory::Memset(SourceToTargetBoneIndexes.GetData(), static_cast<uint8>(INDEX_NONE), SourceNumBones * SourceToTargetBoneIndexes.GetTypeSize());
	for (int32 TargetBoneIndex = 0; TargetBoneIndex < TargetNumBones; ++TargetBoneIndex)
	{
		const int32 SourceBoneIndex = TargetToSourceBoneIndexes[TargetBoneIndex];
		if (SourceBoneIndex != INDEX_NONE)
		{
			SourceToTargetBoneIndexes[SourceBoneIndex] = TargetBoneIndex;
		}
	}
}

const TArray<SmartName::UID_Type>& FSkeletonRemapping::GetSourceToTargetCurveMapping() const
{
	static TArray<SmartName::UID_Type> Dummy;
	return Dummy;
}
