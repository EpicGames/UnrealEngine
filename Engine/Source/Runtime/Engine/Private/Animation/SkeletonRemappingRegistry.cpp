// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/SkeletonRemappingRegistry.h"
#include "Misc/ScopeRWLock.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/Skeleton.h"

namespace UE::Anim
{

FSkeletonRemappingRegistry* GSkeletonRemappingRegistry = nullptr;
FSkeletonRemapping DefaultMapping;

struct FSkeletonRemappingRegistryPrivate
{
	static FDelegateHandle PostGarbageCollectHandle;

	static void HandlePostGarbageCollect()
	{
		// Compact the registry on GC
		if(GSkeletonRemappingRegistry)
		{
			UE::TWriteScopeLock WriteLock(GSkeletonRemappingRegistry->MappingsLock);
			
			for(auto Iter = GSkeletonRemappingRegistry->Mappings.CreateIterator(); Iter; ++Iter)
			{
				const FSkeletonRemappingRegistry::FWeakSkeletonPair& SkeletonPair = Iter.Key();
				if(SkeletonPair.Key.Get() == nullptr || SkeletonPair.Value.Get() == nullptr)
				{
					Iter.RemoveCurrent();
				}
			}

			for(auto Iter = GSkeletonRemappingRegistry->PerSkeletonMappings.CreateIterator(); Iter; ++Iter)
			{
				const TWeakObjectPtr<const USkeleton>& SkeletonPtr = Iter.Key();
				if(SkeletonPtr.Get() == nullptr)
				{
					Iter.RemoveCurrent();
				}
			}
		}
	}
};

FDelegateHandle FSkeletonRemappingRegistryPrivate::PostGarbageCollectHandle;

FSkeletonRemappingRegistry& FSkeletonRemappingRegistry::Get()
{
	checkf(GSkeletonRemappingRegistry, TEXT("Skeleton remapping registry is not instanced. It is only valid to access this while the engine module is loaded."));
	return *GSkeletonRemappingRegistry;
}

void FSkeletonRemappingRegistry::Init()
{
	GSkeletonRemappingRegistry = new FSkeletonRemappingRegistry();
	FSkeletonRemappingRegistryPrivate::PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&FSkeletonRemappingRegistryPrivate::HandlePostGarbageCollect);
}

void FSkeletonRemappingRegistry::Destroy()
{
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(FSkeletonRemappingRegistryPrivate::PostGarbageCollectHandle);
	delete GSkeletonRemappingRegistry;
	GSkeletonRemappingRegistry = nullptr;
}

const FSkeletonRemapping& FSkeletonRemappingRegistry::GetRemapping(const USkeleton* InSourceSkeleton, const USkeleton* InTargetSkeleton)
{
	if (InSourceSkeleton == InTargetSkeleton)
	{
		return DefaultMapping;
	}
	
	const FWeakSkeletonPair Pair(InSourceSkeleton, InTargetSkeleton);
	const uint32 PairHash = GetTypeHash(Pair);

	{
		UE::TReadScopeLock ReadLock(MappingsLock);

		const TSharedPtr<FSkeletonRemapping>* ExistingMapping = Mappings.FindByHash(PairHash, Pair);
		if (ExistingMapping && ExistingMapping->IsValid())
		{
			return *ExistingMapping->Get();
		}
	}

	// No valid mapping was found, so create a new one
	TSharedPtr<FSkeletonRemapping> NewMapping = MakeShared<FSkeletonRemapping>(InSourceSkeleton, InTargetSkeleton);
	
	{
		UE::TWriteScopeLock WriteLock(MappingsLock);

		// First check if another thread grabbed the write lock before us to do the same mapping
		const TSharedPtr<FSkeletonRemapping>* ExistingMapping = Mappings.FindByHash(PairHash, Pair);
		if (ExistingMapping && ExistingMapping->IsValid())
		{
			return *ExistingMapping->Get();
		}

		// Add to global mapping
		Mappings.AddByHash(PairHash, Pair, NewMapping);

		// Add to per-skeleton mappings
		PerSkeletonMappings.Add(InSourceSkeleton, NewMapping);
		PerSkeletonMappings.Add(InTargetSkeleton, NewMapping);

		return *NewMapping;
	}
}

void FSkeletonRemappingRegistry::RefreshMappings(const USkeleton* InSkeleton)
{
	TArray<TSharedPtr<FSkeletonRemapping>> ExistingMappings;
	{
		UE::TWriteScopeLock WriteLock(MappingsLock);
		PerSkeletonMappings.MultiFind(InSkeleton, ExistingMappings);
		
		for(const TSharedPtr<FSkeletonRemapping>& ExistingMapping : ExistingMappings)
		{
			ExistingMapping->RegenerateMapping();
		}
	}
}

}
