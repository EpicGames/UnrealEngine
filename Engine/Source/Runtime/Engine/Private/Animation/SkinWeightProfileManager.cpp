// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/SkinWeightProfileManager.h"
#include "ContentStreaming.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAsset.h"
#include "Rendering/SkeletalMeshRenderData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkinWeightProfileManager)

DEFINE_LOG_CATEGORY_STATIC(LogSkinWeightProfileManager, Warning, Display);

#define PROFILE_SKIN_WEIGHT_MANAGER (!UE_BUILD_SHIPPING)

static int32 GSkipHasPendingInitOrStreamingSkeletalMesh = 1;
static FAutoConsoleVariableRef CVarSkipHasPendingInitOrStreamingSkeletalMesh (
	TEXT("SkinWeightProfileManager.SkipHasPendingInitOrStreamingSkeletalMesh"),
	GSkipHasPendingInitOrStreamingSkeletalMesh,
	TEXT(""),
	ECVF_Default | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarForceSkinWeightProfileManagerAsyncTaskOnGT(
	TEXT("SkinWeightProfileManager.ForceSkinWeightProfileManagerAsyncTaskOnGT"),
	1,
	TEXT(""),
	ECVF_Default | ECVF_RenderThreadSafe
);

static int32 GSkipLODWithoutDataOrStreamingRequest = 0;
static FAutoConsoleVariableRef CVarSkipLODWithoutDataOrStreamingRequest(
	TEXT("SkinWeightProfileManager.SkipLODWithoutDataOrStreamingRequest"),
	GSkipLODWithoutDataOrStreamingRequest,
	TEXT(""),
	ECVF_Default | ECVF_RenderThreadSafe
);

FSkinWeightProfileManager::FSkinWeightProfileManager(UWorld* InWorld) : LastGamethreadProfileIndex(INDEX_NONE), WeakWorld(nullptr)
{
	if (InWorld)
	{
		TickFunction.TickGroup = ETickingGroup::TG_PrePhysics;
		TickFunction.EndTickGroup = ETickingGroup::TG_PostUpdateWork;
		TickFunction.bCanEverTick = true;
		TickFunction.bStartWithTickEnabled = true;
		TickFunction.bTickEvenWhenPaused = false;
		TickFunction.Owner = this;
		TickFunction.RegisterTickFunction(InWorld->PersistentLevel);

		WeakWorld = InWorld;
	}	
}

TMap<UWorld*, FSkinWeightProfileManager*> FSkinWeightProfileManager::WorldManagers;

void FSkinWeightProfileManager::OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	check(WorldManagers.Find(World) == nullptr);

	FSkinWeightProfileManager* NewWorldMan = new FSkinWeightProfileManager(World);
	WorldManagers.Add(World) = NewWorldMan;
}

void FSkinWeightProfileManager::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	if (FSkinWeightProfileManager* *WorldMan = WorldManagers.Find(World))
	{
		delete (*WorldMan);
		WorldManagers.Remove(World);
	}
}

void FSkinWeightProfileManager::OnPreWorldFinishDestroy(UWorld* World)
{
	if (FSkinWeightProfileManager * *WorldMan = WorldManagers.Find(World))
	{
		delete (*WorldMan);
		WorldManagers.Remove(World);
	}
}

void FSkinWeightProfileManager::OnWorldBeginTearDown(UWorld* World)
{
	if (FSkinWeightProfileManager * *WorldMan = WorldManagers.Find(World))
	{
		delete (*WorldMan);
		WorldManagers.Remove(World);
	}
}

FSkinWeightProfileManager* FSkinWeightProfileManager::Get(UWorld* World)
{
	if (FSkinWeightProfileManager * *WorldMan = WorldManagers.Find(World))
	{
		return *WorldMan;
	}

	return nullptr;
}

bool FSkinWeightProfileManager::AllowCPU()
{
	return FSkinWeightProfilesData::AllowCPU();
}

bool FSkinWeightProfileManager::HandleDelayedLoads()
{
	return FSkinWeightProfilesData::HandleDelayedLoads();
}

void FSkinWeightProfileManager::OnStartup()
{
	FWorldDelegates::OnPreWorldInitialization.AddStatic(&OnWorldInit);
	FWorldDelegates::OnPostWorldCleanup.AddStatic(&OnWorldCleanup);
	FWorldDelegates::OnPreWorldFinishDestroy.AddStatic(&OnPreWorldFinishDestroy);
	FWorldDelegates::OnWorldBeginTearDown.AddStatic(&OnWorldBeginTearDown);
}

void FSkinWeightProfileManager::OnShutdown()
{
	for (TPair<UWorld*, FSkinWeightProfileManager*>& Pair : WorldManagers)
	{
		if (Pair.Value)
		{
			delete Pair.Value;
		}
	}
	WorldManagers.Empty();
}

void FSkinWeightProfileManager::RequestSkinWeightProfileStack(
	FSkinWeightProfileStack InProfileStack, 
	USkinnedAsset* SkinnedAsset, 
	UObject* Requester,
	FRequestFinished& Callback,
	int32 LODIndex /*= INDEX_NONE*/
	)
{
	// Make sure we have an actual skeletal mesh
	if (USkeletalMesh* const Mesh = Cast<USkeletalMesh>(SkinnedAsset))
	{
		// Setup a request structure
		FSetProfileRequest ProfileRequest;
		ProfileRequest.ProfileStack = InProfileStack.Normalized();
		ProfileRequest.Callback = Callback;
		ProfileRequest.WeakSkeletalMesh = Mesh;
		ProfileRequest.IdentifyingObject = Requester;

		UE_LOGF(LogSkinWeightProfileManager, Display, "RequestSkinWeightProfile [%ls | %ls]", *Mesh->GetName(), *Requester->GetName());

		if (LODIndex == INDEX_NONE)
		{
			// Match what rendering does.
			constexpr bool bForceLowestLODIdx = true;
			const int32 NumLODS = Mesh->GetLODNum();
			for (int32 index = Mesh->GetMinLodIdx(bForceLowestLODIdx); index < NumLODS; ++index)
			{
				ProfileRequest.LODIndices.Add(index);
			}
		}
		else
		{
			ProfileRequest.LODIndices.Add(LODIndex);
		}
		
		// Add the profile request
		const int32 Index = PendingSetProfileRequests.Add(ProfileRequest);

		// Check whether or not we might have already setup a readback for this skeletal mesh, and add a request to it
		int32& NumRequestsForMesh = PendingMeshes.FindOrAdd(Mesh, 0);
		++NumRequestsForMesh;

		WaitingForStreaming = false;

		TickFunction.SetTickFunctionEnable(true);
	}
}

void FSkinWeightProfileManager::CancelSkinWeightProfileRequest(UObject* Requester)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SkinWeightProfileManager_CancelSkinWeightProfileRequest);

	for (FSetProfileRequest& Request : PendingSetProfileRequests)
	{
		if (Request.IdentifyingObject == Requester || Request.WeakSkeletalMesh == Requester)
		{
			WaitingForStreaming = false;
			CanceledRequest.Add(Request);
			UE_LOGF(LogSkinWeightProfileManager, Display, "Cancel [%ls]", *Requester->GetName());			
		}
	}
}

void FSkinWeightProfileManager::DoTick(float DeltaTime, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{	
#if PROFILE_SKIN_WEIGHT_MANAGER 
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(SkinWeightProfileManagerGT);	
	CSV_CUSTOM_STAT_GLOBAL(SkinWeightProfileManager_PendingMeshes, PendingMeshes.Num(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_GLOBAL(SkinWeightProfileManager_PendingRequests, PendingSetProfileRequests.Num(), ECsvCustomStatOp::Set);
#endif // PROFILE_SKIN_WEIGHT_MANAGER 

	QUICK_SCOPE_CYCLE_COUNTER(STAT_SkinWeightProfileManager_Tick);

	LastGamethreadProfileIndex = INDEX_NONE;

	if (PendingSetProfileRequests.Num())
	{
		if (ShouldSkipTick())
		{
			return;
		}

		WaitingForStreaming = false;

		if (AsyncTask.IsValid())
		{
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(AsyncTask, ENamedThreads::GameThread);
			AsyncTask.SafeRelease();
		}

		// Release stream-out protection for meshes whose readback has progressed past
		// the danger window
		TickReleaseStreamOutProtection_GameThread();

		bool bInProgressRequestsExist = false;
		for (int32 RequestIndex = 0; RequestIndex < PendingSetProfileRequests.Num(); ++RequestIndex)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_SkinWeightProfileManager_TickMesh);

			FSetProfileRequest& Request = PendingSetProfileRequests[RequestIndex];

			Request.ExpectedResidentLODs = INDEX_NONE;
			
			// Remove if canceled or objects have gone stale 
			bool bRemove = Request.IdentifyingObject.IsStale() || Request.WeakSkeletalMesh.IsStale() ||
				CanceledRequest.ContainsByPredicate([Request](FSetProfileRequest& B)
			{
				return Request.IdentifyingObject == B.IdentifyingObject && Request.WeakSkeletalMesh == B.WeakSkeletalMesh && Request.ProfileStack == B.ProfileStack;
			});

			const USkeletalMesh* SkeletalMesh = Request.WeakSkeletalMesh.Get();
			FSkeletalMeshRenderData* RenderData = SkeletalMesh ? SkeletalMesh->GetResourceForRendering() : nullptr;
			// Or if skeletal mesh / render data is invalid
			bRemove = bRemove ? true : (RenderData == nullptr);

			if (!bRemove)
			{	
				check(SkeletalMesh);
				bool bAllBuffersReady = true;
				int32 MinResidentLODsToWaitFor = INDEX_NONE;
				for (int32 LODProfileArrayIndex = 0; LODProfileArrayIndex < Request.LODIndices.Num(); ++LODProfileArrayIndex)
				{
					int32 LODRenderDataArrayIndex = Request.LODIndices[LODProfileArrayIndex];
					const FSkinWeightProfilesData& SkinWeightProfilesData = RenderData->LODRenderData[LODRenderDataArrayIndex].SkinWeightProfilesData;
					if (!SkinWeightProfilesData.ContainsOverrideBuffer(Request.ProfileStack))
					{
						bAllBuffersReady = false;
						int32 NumResidentLODsRequired = ConvertLODIndexToCount(*SkeletalMesh, LODRenderDataArrayIndex);
						if (ShouldIgnoreLOD(SkinWeightProfilesData, *SkeletalMesh, NumResidentLODsRequired))
						{
							// This LOD isn't streamed in and has no requests for that. There may be several such LODs for the current model.
							// We will create a record in StreamingBlockedMeshes to track when any of these LODs will be streamed in
							MinResidentLODsToWaitFor = (MinResidentLODsToWaitFor == INDEX_NONE) ?
								NumResidentLODsRequired :
								FMath::Min(MinResidentLODsToWaitFor, NumResidentLODsRequired);
						}
						else
						{
							bInProgressRequestsExist = true;
							if (!GSkipLODWithoutDataOrStreamingRequest)
							{
								break;
							}
						}
					}
				}

				Request.ExpectedResidentLODs = MinResidentLODsToWaitFor;

				if (bAllBuffersReady)
				{
					if (Request.IdentifyingObject.IsValid())
					{
						UE_LOGF(LogSkinWeightProfileManager, Display, "Callback [%ls | %ls]", *SkeletalMesh->GetName(), *Request.IdentifyingObject->GetName());
						Request.Callback(Request.WeakSkeletalMesh, Request.ProfileStack);
						bRemove = true;
					}
				}
			}
			
			if (bRemove)
			{
				CleanupRequest(Request);
				CanceledRequest.Remove(Request);
				PendingSetProfileRequests.RemoveAtSwap(RequestIndex);
				--RequestIndex;
			}
		}

		if (bInProgressRequestsExist || !GSkipLODWithoutDataOrStreamingRequest)
		{
			// This is the last index that we will process asynchronously, while the task is running we do not remove request from the array so access should be safe
			LastGamethreadProfileIndex = PendingSetProfileRequests.Num() - 1;

			if (MyCompletionGraphEvent)
			{
				AsyncTask = TGraphTask<FSkinWeightProfileManagerAsyncTask>::CreateTask(nullptr, CurrentThread).ConstructAndDispatchWhenReady(this);
				TickFunction.GetCompletionHandle()->DontCompleteUntil(AsyncTask);
			}
		}
		else
		{
			// This is set when we still have pending requests, but all their unprocessed LODs have not been streamed in.
			WaitingForStreaming = true;
		}
	}
	else
	{
		// Drain any leftover protections even if no requests are pending this tick.
		TickReleaseStreamOutProtection_GameThread();
		TickFunction.SetTickFunctionEnable(false);
	}
}

void FSkinWeightProfileManager::RegisterStreamOutProtection_AsyncSafe(USkinnedAsset* SkinnedAsset)
{
	if (!SkinnedAsset)
	{
		return;
	}
	FScopeLock Lock(&ProtectedSkinnedAssetsCS);
	ProtectedSkinnedAssets.Add(TWeakObjectPtr<USkinnedAsset>(SkinnedAsset));
}

void FSkinWeightProfileManager::TickReleaseStreamOutProtection_GameThread()
{
	using EReadbackStatus = FSkinWeightProfilesData::EReadbackStatus;

	FScopeLock Lock(&ProtectedSkinnedAssetsCS);
	if (ProtectedSkinnedAssets.IsEmpty())
	{
		return;
	}

	IRenderAssetStreamingManager& RAMgr = IStreamingManager::Get().GetRenderAssetStreamingManager();

	for (auto It = ProtectedSkinnedAssets.CreateIterator(); It; ++It)
	{
		USkinnedAsset* Asset = It->Get();

		// GC'd: streaming manager already cleaned up via RemoveStreamingRenderAsset.
		if (!Asset)
		{
			It.RemoveCurrent();
			continue;
		}

		const FSkeletalMeshRenderData* RenderData = Asset->GetResourceForRendering();
		if (!RenderData)
		{
			RAMgr.UnregisterStreamOutProtectedAsset(Asset);
			It.RemoveCurrent();
			continue;
		}

		// Protection must hold across two windows:
		//  1. PendingGPUReadback   — set on GT/Async; the RT lambda has not yet recorded
		//                            FRHICommandCopyToStagingBuffer onto the RHI cmd list.
		//  2. GPUReadbackEnqueued  — RT lambda recorded the Copy, but the RHI thread has
		//                            not executed it yet (and the GPU certainly hasn't
		//                            completed it). A StreamOut-batched Replace landing on
		//                            the cmd list before the Copy executes nulls the
		//                            source's UnderlyingResource and crashes the readback.
		//
		// Status transitions GPUReadbackEnqueued -> DataReadbackEnqueued only after
		// BufferReadback->IsReady() (RHI fence signaled, GPU done with the Copy). Past
		// that point the staging buffer holds the data and the source can be replaced.
		bool bAnyStillPending = false;
		for (const FSkeletalMeshLODRenderData& LODData : RenderData->LODRenderData)
		{
			const EReadbackStatus Status = LODData.SkinWeightProfilesData.GetReadbackStatus();
			if (Status == EReadbackStatus::PendingGPUReadback ||
				Status == EReadbackStatus::GPUReadbackEnqueued)
			{
				bAnyStillPending = true;
				break;
			}
		}

		if (!bAnyStillPending)
		{
			RAMgr.UnregisterStreamOutProtectedAsset(Asset);
			It.RemoveCurrent();
		}
	}
}

void FSkinWeightProfileManager::Tick(float DeltaTime)
{
	DoTick(DeltaTime, ENamedThreads::GameThread, nullptr);
}

bool FSkinWeightProfileManager::IsTickable() const
{
	return WeakWorld.IsExplicitlyNull();
}

TStatId FSkinWeightProfileManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FSkinWeightProfileManager, STATGROUP_Tickables);
}

void FSkinWeightProfileManager::CleanupRequest(const FSetProfileRequest& Request)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SkinWeightProfileManager_CleanupRequest);
	// Remove request for its pending skeletal mesh
	if (int32* RequestsPtr = PendingMeshes.Find(Request.WeakSkeletalMesh))
	{
		int32& NumPendingRequests = *RequestsPtr;
		--NumPendingRequests;

		// In case all requests have finished clean up the readback data and remove skeletal mesh as pending
		if (NumPendingRequests == 0)
		{
			PendingMeshes.Remove(Request.WeakSkeletalMesh);

			if (USkeletalMesh * SkeletalMesh = Request.WeakSkeletalMesh.Get())
			{
				if (FSkeletalMeshRenderData * RenderData = SkeletalMesh->GetResourceForRendering())
				{
					for (FSkeletalMeshLODRenderData& LODRenderData : RenderData->LODRenderData)
					{
						FSkinWeightProfilesData& SkinweightData = LODRenderData.SkinWeightProfilesData;
						SkinweightData.ResetGPUReadback();
					}
				}
			}
		}
	}
	
}

bool FSkinWeightProfileManager::ShouldIgnoreLOD(const FSkinWeightProfilesData& SkinWeightProfilesData, const USkeletalMesh& SkeletalMesh, int32 NumResidentLODsRequired) const
{
	if (!GSkipLODWithoutDataOrStreamingRequest)
	{
		return false;
	}

	// Do we have enough LODs in memory?
	const FStreamableRenderResourceState& SRRState = SkeletalMesh.GetStreamableResourceState();
	if (SRRState.NumResidentLODs >= NumResidentLODsRequired)
	{
		return false;
	}

	// Are they scheduled to be streamed in?
	if (SkeletalMesh.HasPendingInitOrStreaming())
	{
		return false;
	}

	// This mesh LOD isn't loaded and not scheduled for streaming, we can ignore it
	return true;
}

// Used to return early from DoTick if the entire request queue consists of LOD meshes that have not yet been streamed in.
bool FSkinWeightProfileManager::ShouldSkipTick() const
{
	if (!WaitingForStreaming || PendingSetProfileRequests.IsEmpty() || !GSkipLODWithoutDataOrStreamingRequest)
	{
		return false;
	}

	for (const FSetProfileRequest& Request : PendingSetProfileRequests)
	{
		if (Request.ExpectedResidentLODs == INDEX_NONE || Request.IdentifyingObject.IsStale())
		{
			return false; // Continue ticking because there are stale requests to clean up
		}

		const USkeletalMesh* Mesh = Request.WeakSkeletalMesh.Get();
		if (!Mesh || (Mesh->GetStreamableResourceState().NumResidentLODs >= Request.ExpectedResidentLODs))
		{
			return false; // Continue ticking to clean up or because something was streamed in
		}
	}

	return true; // All current requests are still waiting for streaming, we can end current tick early
}

int32 FSkinWeightProfileManager::ConvertLODIndexToCount(const USkeletalMesh& SkeletalMesh, int32 Index) const
{
	const FStreamableRenderResourceState& SRRState = SkeletalMesh.GetStreamableResourceState();
	return (SRRState.MaxNumLODs - (Index - SRRState.AssetLODBias));
}

bool FSkinWeightProfileManager::IsTickableWhenPaused() const
{
	return false;
}

bool FSkinWeightProfileManager::IsTickableInEditor() const
{
	return true;
}

void FSkinWeightProfileManagerTickFunction::ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	Owner->DoTick(DeltaTime, CurrentThread, MyCompletionGraphEvent);
}

FString FSkinWeightProfileManagerTickFunction::DiagnosticMessage()
{
	return TEXT("FSkinWeightProfileManagerTickFunction::Tick");
}

FName FSkinWeightProfileManagerTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName("FSkinWeightProfileManager");
}

ENamedThreads::Type FSkinWeightProfileManagerAsyncTask::GetDesiredThread()
{
#if WITH_EDITOR
	// Force task to run on GT since it can cause a stall waiting for RT
	return ENamedThreads::GameThread;
#else

	if (CVarForceSkinWeightProfileManagerAsyncTaskOnGT.GetValueOnAnyThread())
	{
    	return ENamedThreads::GameThread;
	}

#if PLATFORM_ANDROID
	    return IsRunningRHIInSeparateThread() ? ENamedThreads::RHIThread : ENamedThreads::ActualRenderingThread;
#else
		return ENamedThreads::AnyBackgroundHiPriTask;
#endif

#endif 		
}


void FSkinWeightProfileManagerAsyncTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
#if PROFILE_SKIN_WEIGHT_MANAGER 
	CSV_SCOPED_TIMING_STAT_GLOBAL(SkinWeightProfileManagerAsync);
#endif // PROFILE_SKIN_WEIGHT_MANAGER 
	
	const TArray<FSetProfileRequest>& OwnerRequests = Owner->PendingSetProfileRequests;

	checkf(Owner->LastGamethreadProfileIndex < OwnerRequests.Num(), TEXT("Invalid state"));

	for (int32 RequestIndex = 0; RequestIndex <= Owner->LastGamethreadProfileIndex; ++RequestIndex)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_SkinWeightProfileManager_TickMeshAsync);

		const FSetProfileRequest& Request = OwnerRequests[RequestIndex];
		if (const USkeletalMesh* SkeletalMesh = Request.WeakSkeletalMesh.Get())
		{
			if (GSkipHasPendingInitOrStreamingSkeletalMesh && SkeletalMesh->HasPendingInitOrStreaming()) continue;

			if (FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering())
			{
				for (const int32 LODIndex : Request.LODIndices)
				{
					FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LODIndex];
					FSkinWeightProfilesData& SkinweightData = LODRenderData.SkinWeightProfilesData;

					// Don't try to initialize or do a GPU readback for a profile that's already completed.
					if (FSkinWeightProfileManager::HandleDelayedLoads() && SkinweightData.HasProfileStack(Request.ProfileStack))
					{
						continue;
					}
					
					// If we have CPU-side weights available, use them right away.
					if (FSkinWeightProfileManager::AllowCPU() &&
						((FSkinWeightProfileManager::HandleDelayedLoads() && LODRenderData.SkinWeightVertexBuffer.GetDataVertexBuffer()->IsWeightDataValid()) ||
						(!FSkinWeightProfileManager::HandleDelayedLoads() && LODRenderData.SkinWeightVertexBuffer.GetNeedsCPUAccess())))
					{
						SkinweightData.InitialiseProfileBuffer(Request.ProfileStack);
					}
					else
					{
						if (SkinweightData.IsPendingReadback())
						{
							if (!IStreamingManager::Get_Concurrent() || !IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::SkeletalMesh))
							{
								SkinweightData.EnqueueGPUReadback();
							}
							else
							{
								// Atomically test "no pending streaming op" and protect against
								// StreamOut for the duration of the readback. 
								USkeletalMesh* MutableMesh = const_cast<USkeletalMesh*>(SkeletalMesh);
								IRenderAssetStreamingManager& RAMgr = IStreamingManager::Get().GetRenderAssetStreamingManager();
								if (RAMgr.TryRegisterStreamOutProtectedAsset(MutableMesh))
								{
									SkinweightData.EnqueueGPUReadback();
									Owner->RegisterStreamOutProtection_AsyncSafe(MutableMesh);
								}
								// else: pending streaming op in flight; leave status at None and retry next tick.
							}
							
						}
						else if (SkinweightData.IsGPUReadbackFinished())
						{
							if (SkinweightData.IsGPUReadbackFinished() && !SkinweightData.IsDataReadbackPending())
							{						
								SkinweightData.EnqueueDataReadback();
							}
							else if (SkinweightData.IsDataReadbackFinished())
							{						
								SkinweightData.InitialiseProfileBuffer(Request.ProfileStack);
							}
						}
					}
				}
			}
		}
	}
}
