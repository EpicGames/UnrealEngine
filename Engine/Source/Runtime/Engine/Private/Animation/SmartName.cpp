// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/SmartName.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/PropertyPortFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SmartName)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

////////////////////////////////////////////////////////////////////////
//
// FSmartNameMapping
//
///////////////////////////////////////////////////////////////////////
FSmartNameMapping::FSmartNameMapping()
{
}

void FSmartNameMapping::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::SmartNameRefactor)
	{
		if (Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::SmartNameRefactorForDeterministicCooking)
		{
			TMap<FName, FGuid> TempGuidMap;
			Ar << TempGuidMap;
		}
	}
	else if (Ar.UEVer() >= VER_UE4_SKELETON_ADD_SMARTNAMES)
	{
		SmartName::UID_Type NextUidTemp;
		Ar << NextUidTemp;

		TMap<SmartName::UID_Type, FName> TempUidMap;
		Ar << TempUidMap;
	}

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::MoveCurveTypesToSkeleton)
	{
		Ar << CurveMetaDataMap;
	}

	if (Ar.IsLoading())
	{
		CurveMetaDataMap.GenerateKeyArray(CurveNameList);
#if !WITH_EDITOR
		CurveMetaDataMap.GenerateValueArray(CurveMetaDataList);
#endif
	}
}

FArchive& operator<<(FArchive& Ar, FSmartNameMapping& Elem)
{
	Elem.Serialize(Ar);

	return Ar;
}

////////////////////////////////////////////////////////////////////////
//
// FSmartNameContainer
//
//////////////////////////////////////////////////////////////////////
FSmartNameMapping* FSmartNameContainer::GetContainerInternal(const FName& ContainerName)
{
	return NameMappings.Find(ContainerName);
}

const FSmartNameMapping* FSmartNameContainer::GetContainerInternal(const FName& ContainerName) const
{
	return NameMappings.Find(ContainerName);
}

void FSmartNameContainer::Serialize(FArchive& Ar, bool bIsTemplate)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsCooking() && !bIsTemplate)
	{
		Ar << LoadedNameMappings;
	}
	else
#endif
	{
		Ar << NameMappings;
	}
}

////////////////////////////////////////////////////////////////////////
//
// FSmartName
//
///////////////////////////////////////////////////////////////////////
bool FSmartName::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	Ar << DisplayName;
	if (Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::RemoveUIDFromSmartNameSerialize)
	{
		SmartName::UID_Type TempUID;
		Ar << TempUID;
	}
#if WITH_EDITOR
	else if (Ar.IsTransacting() || Ar.HasAnyPortFlags(PPF_Duplicate))
	{
		Ar << UID;
	}
#endif

	// only save if it's editor build and not cooking
	if (Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::SmartNameRefactorForDeterministicCooking)
	{
		FGuid TempGUID;
		Ar << TempGUID;
	}

	return true;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

