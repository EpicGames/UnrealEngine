// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/TimeStretchCurve.h"
#include "Animation/AnimCurveTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimeStretchCurve)

bool FTimeStretchCurve::IsValid() const
{
	return Markers.Num() > 0;
}

void FTimeStretchCurve::Reset()
{
	Markers.Reset();
}

void FTimeStretchCurve::BakeFromFloatCurve(const FFloatCurve& TimeStretchCurve, float InSequenceLength)
{
	// Figure out how many steps this is going to take, 
	// as we want to sample this curve with a fixed time step, across the whole length of the animation.
	const float DesiredSamplingRate = FMath::Clamp(SamplingRate, 1.f, 240.f);
	const int32 NumTimeStretchCurveSegments = FMath::Max(FMath::FloorToInt(InSequenceLength * DesiredSamplingRate), 1);

	// Find actual SamplingTimeStep based on number of NumSegments.
	// We want to cover the entire SequenceLength with fixed time steps. 
	// So we can't exactly matching the sampling rate.
	const float SamplingTimeStep = InSequenceLength / (float)NumTimeStretchCurveSegments;

	// Sample curve at given time steps.
	float MaxValue = 0.f;
	for (int32 SegmentIndex = 0; SegmentIndex < NumTimeStretchCurveSegments; SegmentIndex++)
	{
		const float EvaluationTime = (float)SegmentIndex * SamplingTimeStep;
		const float CurveValue = FMath::Max(TimeStretchCurve.Evaluate(EvaluationTime), 0.f);
		Markers.Add(FTimeStretchCurveMarker(EvaluationTime, CurveValue));

		if (CurveValue > MaxValue)
		{
			MaxValue = CurveValue;
		}
	}

	// If Max Value is near zero, we have no valid time stretching curve.
	if (MaxValue < UE_KINDA_SMALL_NUMBER)
	{
		Reset();
		return;
	}

	// Normalize Samples.
	for (FTimeStretchCurveMarker& CurrMarker : Markers)
	{
		CurrMarker.Alpha /= MaxValue;
	}

	// Optimize TimeStretchMarkers (Remove near value keys)
	// Don't trim last marker, we need it to describe the end of the animation.
	{
		const int32 NumSegments = Markers.Num();
		int32 MarkerIndex = 0;
		for (int32 SegmentIndex = 0; SegmentIndex < NumSegments - 2; SegmentIndex++)
		{
			const FTimeStretchCurveMarker& CurrMarker = Markers[MarkerIndex + 0];
			const FTimeStretchCurveMarker& NextMarker = Markers[MarkerIndex + 1];

			if (FMath::IsNearlyEqual(CurrMarker.Alpha, NextMarker.Alpha, CurveValueMinPrecision))
			{
				Markers.RemoveAt(MarkerIndex + 1, EAllowShrinking::No);
			}
			else
			{
				MarkerIndex++;
			}
		}

		Markers.Shrink();

		// We need to have more than 2 markers to do anything interesting.
		// 2 markers means start and end, and a constant value of 1.
		if (Markers.Num() <= 2)
		{
			Reset();
			return;
		}
	}

	// Cache upper and lowers bounds.
	// This is basically the most scaling we can get out of the curve, 
	// and therefore the shortest or longest play back time we can get without using uniform scaling.
	{
		// No uniform scaling
		const float U = 1.f;

		// We don't want S to get too big or we risk running into precision issues.
		const float S_Max = 100.f;

		// We don't want to get too close to -1 here, for precision issues
		// And a value of -1 means animation play back is paused.
		const float S_Min = -1.f + 0.01f;

		float P_Target_Min = 0.f;
		float P_Target_Max = 0.f;
		Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_Original] = 0.f;
		Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_TargetMin] = 0.f;
		Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_TargetMax] = 0.f;

		const int32 NumMarkers = Markers.Num();
		for (int32 MarkerIndex = 0; MarkerIndex < NumMarkers - 1; MarkerIndex++)
		{
			FTimeStretchCurveMarker& CurrMarker = Markers[MarkerIndex];

			CurrMarker.Time[(uint8)ETimeStretchCurveMapping::T_TargetMin] = P_Target_Min;
			CurrMarker.Time[(uint8)ETimeStretchCurveMapping::T_TargetMax] = P_Target_Max;

			const float CurrMarkerT_Original = CurrMarker.Time[(uint8)ETimeStretchCurveMapping::T_Original];
			const float NextMarkerT_Original = Markers[MarkerIndex + 1].Time[(uint8)ETimeStretchCurveMapping::T_Original];

			const float dT_Original_i = (NextMarkerT_Original - CurrMarkerT_Original);
			const float C_i = CurrMarker.Alpha;
			Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_Original] += dT_Original_i * C_i;

			// Cache lower bound
			{
				const float PlayRate_TargetMin_i = (U * (1.f + S_Max * C_i));
				P_Target_Min += dT_Original_i / PlayRate_TargetMin_i;

				const float CurrMarkerT_TargetMin = CurrMarker.Time[(uint8)ETimeStretchCurveMapping::T_TargetMin];
				const float NextMarkerT_TargetMin = P_Target_Min;
				const float dT_TargetMin_i = (NextMarkerT_TargetMin - CurrMarkerT_TargetMin);
				Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_TargetMin] += dT_TargetMin_i * C_i;
			}

			// Cache upper bound
			{
				const float PlayRate_TargetMax_i = (U * (1.f + S_Min * C_i));
				P_Target_Max += dT_Original_i / PlayRate_TargetMax_i;

				const float CurrMarkerT_TargetMax = CurrMarker.Time[(uint8)ETimeStretchCurveMapping::T_TargetMax];
				const float NextMarkerT_TargetMax = P_Target_Max;
				const float dT_TargetMax_i = (NextMarkerT_TargetMax - CurrMarkerT_TargetMax);
				Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_TargetMax] += dT_TargetMax_i * C_i;
			}
		}

		FTimeStretchCurveMarker& LastMarker = Markers[NumMarkers - 1];
		LastMarker.Time[(uint8)ETimeStretchCurveMapping::T_Original] = InSequenceLength;
		LastMarker.Time[(uint8)ETimeStretchCurveMapping::T_TargetMin] = P_Target_Min;
		LastMarker.Time[(uint8)ETimeStretchCurveMapping::T_TargetMax] = P_Target_Max;
	}

	// Validate our cached data. 
	// If we don't have valid data, abort.
	bool bHasValidData = true;
	{
		const FTimeStretchCurveMarker& LastMarker = Markers.Last();

		const float T_Original = LastMarker.Time[(uint8)ETimeStretchCurveMapping::T_Original];
		const float T_TargetMin = LastMarker.Time[(uint8)ETimeStretchCurveMapping::T_TargetMin];
		const float T_TargetMax = LastMarker.Time[(uint8)ETimeStretchCurveMapping::T_TargetMax];

		// Cached target bounds should not be zero, or we can't remap them in 'ConditionallyUpdateCachedData'
		// to our desired target time.
		if (FMath::IsNearlyZero(T_TargetMin) || FMath::IsNearlyZero(T_TargetMax))
		{
			bHasValidData = false;
		}

		// Similarly, if our bounds are too close to T_Original, we can't do our remapping either.
		if (FMath::IsNearlyEqual(T_Original, T_TargetMin) || FMath::IsNearlyEqual(T_Original, T_TargetMax))
		{
			bHasValidData = false;
		}
	}

	if (!bHasValidData)
	{
		Reset();
	}
}

void FTimeStretchCurveInstance::InitializeFromPlayRate(float InPlayRate, const FTimeStretchCurve& TimeStretchCurve)
{
	// This is set to true at the end, if initialization is successful.
	bHasValidData = false;
	if (!TimeStretchCurve.IsValid() || FMath::IsNearlyZero(InPlayRate))
	{
		return;
	}

	T_Original = TimeStretchCurve.Markers.Last().Time[(uint8)ETimeStretchCurveMapping::T_Original];
	T_Target = T_Original / FMath::Abs(InPlayRate);

	// See if T_Target falls in a range we have already mapped.
	// If not, we need uniform scaling U to help us out.
	float Alpha;
	float U;
	ETimeStretchCurveMapping CachedBoundType;

	const int32 NumMarkers = TimeStretchCurve.Markers.Num();
	const int32 LastMarkerIndex = NumMarkers - 1;

	if (T_Target < T_Original)
	{
		const float T_TargetMin = TimeStretchCurve.Markers[LastMarkerIndex].Time[(uint8)ETimeStretchCurveMapping::T_TargetMin];
		if (T_Target < T_TargetMin)
		{
			// Make sure we don't divide by zero. This should not have been allowed at curve creation time.
			check(!FMath::IsNearlyZero(T_TargetMin));
			U = T_Target / T_TargetMin;
			Alpha = 1.f;
		}
		else
		{
			// Make sure we don't divide by zero. This should not have been allowed at curve creation time.
			check(!FMath::IsNearlyEqual(T_Original, T_TargetMin));
			U = 1.f;
			Alpha = (T_Original - T_Target) / (T_Original - T_TargetMin);
		}

		CachedBoundType = ETimeStretchCurveMapping::T_TargetMin;
	}
	else
	{
		const float T_TargetMax = TimeStretchCurve.Markers[LastMarkerIndex].Time[(uint8)ETimeStretchCurveMapping::T_TargetMax];
		if (T_Target > T_TargetMax)
		{
			// Make sure we don't divide by zero. This should not have been allowed at curve creation time.
			check(!FMath::IsNearlyZero(T_TargetMax));
			U = T_Target / T_TargetMax;
			Alpha = 1.f;
		}
		else
		{
			// Make sure we don't divide by zero. This should not have been allowed at curve creation time.
			check(!FMath::IsNearlyEqual(T_Original, T_TargetMax));
			U = 1.f;
			Alpha = (T_Target - T_Original) / (T_TargetMax - T_Original);
		}

		CachedBoundType = ETimeStretchCurveMapping::T_TargetMax;
	}

	/*
	Cache markers mapped in Target space.
	This essentially maps them in 'play back time' space
	We can use linear interpolation between our original curve, and the bounds we have mapped.
	This gives a good approximation of where the makers should be in play back space.
	*/
	{
		P_Marker_Target.Reset(NumMarkers);
		P_Marker_Target.AddUninitialized(NumMarkers);
		for (int32 MarkerIndex = 0; MarkerIndex < LastMarkerIndex; MarkerIndex++)
		{
			const FTimeStretchCurveMarker& CurrMarker = TimeStretchCurve.Markers[MarkerIndex];
			const float MarkerT_Original = CurrMarker.Time[(uint8)ETimeStretchCurveMapping::T_Original];
			const float MarkerT_TargetBound = CurrMarker.Time[(uint8)CachedBoundType];

			const float MarkerT_Target = U * FMath::LerpStable<float>(MarkerT_Original, MarkerT_TargetBound, Alpha);
			P_Marker_Target[MarkerIndex] = MarkerT_Target;
		}

		// Set to end exactly, no precision error.
		P_Marker_Target[LastMarkerIndex] = T_Target;
	}

	/*
		Calculate S

		Starting with dTO_i = dT_i * U * (1 + S * C_i)
		after summing that over the N markers that we have, we end up with:
		S = (T_Original - T_Target * U) / (U * Sum(dT_i * C_i))

		'Sum_dT_i_by_C_i' is something we also have precomputed, and can approximate via linear interpolation.
	*/
	const float Sum_dT_i_by_C_i_Original = TimeStretchCurve.Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_Original];
	const float Sum_dT_i_by_C_i_TargetBound = TimeStretchCurve.Sum_dT_i_by_C_i[(uint8)CachedBoundType];
	const float Sum_dT_i_by_C_i_Target = U * FMath::LerpStable<float>(Sum_dT_i_by_C_i_Original, Sum_dT_i_by_C_i_TargetBound, Alpha);

	const float U_by_Sum_dT_i_by_C_i_Target = U * Sum_dT_i_by_C_i_Target;
	if (FMath::IsNearlyZero(U_by_Sum_dT_i_by_C_i_Target))
	{
		return;
	}
	const float S = (T_Original - T_Target * U) / U_by_Sum_dT_i_by_C_i_Target;

	/**
		S should always be greater than -1. As -1 would pause, and less would play in reverse.
		This will occur if er are outside of our cached bounds,
		and have to extrapolate with very high U.
	*/
	if (S <= -1.f)
	{
		return;
	}

	/**
		If our OverallPlayRate is too small, abort and don't use the Time Stretch Curve.
	*/
	const float OverallPlayRate = U * (1.f + S);
	if (OverallPlayRate < UE_SMALL_NUMBER)
	{
		return;
	}

	// Sanity check we're not dealing with bad numbers.
	check(FMath::IsFinite(S) && !FMath::IsNaN(S));
	check(FMath::IsFinite(U) && !FMath::IsNaN(U));

	/*
		Since our mapping in Target space is approximated. (linear interp between original and bounds)
		But also because U may have some influence,
		It won't exactly match our original authored markers.
		So we need to remap our markers in Original space, based off the Target mapping we'll be using.
	*/
	{
		P_Marker_Original.Reset(NumMarkers);
		P_Marker_Original.AddUninitialized(NumMarkers);
		float MarkerP_Original = 0.f;

		for (int32 MarkerIndex = 0; MarkerIndex < LastMarkerIndex; MarkerIndex++)
		{
			P_Marker_Original[MarkerIndex] = MarkerP_Original;

			const float P_CurrMarker_Target = P_Marker_Target[MarkerIndex + 0];
			const float P_NextMarker_Target = P_Marker_Target[MarkerIndex + 1];

			const float dT_Target_i = (P_NextMarker_Target - P_CurrMarker_Target);
			const float C_i = TimeStretchCurve.Markers[MarkerIndex].Alpha;

			MarkerP_Original += dT_Target_i * U * (1.f + S * C_i);
		}

		// Set exact end time.
		P_Marker_Original[LastMarkerIndex] = T_Original;
	}

	/*
		Since we need to map relative positions in Original and Target space,
		We require markers to not be overlapping.
		Trim any markers too close to each other.
		We do this on both P_Marker_Target and P_Marker_Original, as they need to stay in sync.
		We also exclude first and last markers, as we need these to describe beginning and ending positions.
		We don't worry about these potentially overlapping, as we check below that we have at least 3 markers.
	*/
	{
		for (int32 MarkerIndex = 0; MarkerIndex < P_Marker_Original.Num() - 1; MarkerIndex++)
		{
			const float P_CurrMarker_Target = P_Marker_Target[MarkerIndex + 0];
			const float P_NextMarker_Target = P_Marker_Target[MarkerIndex + 1];

			const float P_CurrMarker_Original = P_Marker_Original[MarkerIndex + 0];
			const float P_NextMarker_Original = P_Marker_Original[MarkerIndex + 1];

			const float dT_Target = (P_NextMarker_Target - P_CurrMarker_Target);
			const float dT_Original = (P_NextMarker_Original - P_CurrMarker_Original);

			if (FMath::IsNearlyZero(dT_Target, UE_KINDA_SMALL_NUMBER) || FMath::IsNearlyZero(dT_Original, UE_KINDA_SMALL_NUMBER))
			{
				P_Marker_Target.RemoveAt(MarkerIndex + 1, EAllowShrinking::No);
				P_Marker_Original.RemoveAt(MarkerIndex + 1, EAllowShrinking::No);
				MarkerIndex--;
			}
		}

		// We need more than 2 markers to do anything interesting.
		// Since 2 markers would be a constant curve of 1.
		if (P_Marker_Original.Num() <= 2)
		{
			return;
		}
	}

	// Successful initialization
	bHasValidData = true;
}

void FTimeStretchCurveInstance::UpdateMarkerIndexForPosition(int32& InOutMarkerIndex, float InPosition, const TArray<float>& InMarkerPositions) const
{
	check(bHasValidData);

	// If maker is invalid, binary search new match
	if ((InOutMarkerIndex < 0) || (InOutMarkerIndex > (InMarkerPositions.Num() - 2)))
	{
		InOutMarkerIndex = BinarySearchMarkerIndex(InPosition, InMarkerPositions);
		return;
	}

	// Then, test if we're still with the same marker, then we don't need to perform any work.
	if (IsValidMarkerForPosition(InOutMarkerIndex, InPosition, InMarkerPositions))
	{
		return;
	}

	// Otherwise do a binary search.
	// @todo binary search will take at most Log2(N) steps.
	// Since animation tends to move linearly we could check ahead, to see if linear search would be more efficient.
	InOutMarkerIndex = BinarySearchMarkerIndex(InPosition, InMarkerPositions);
}

bool FTimeStretchCurveInstance::IsValidMarkerForPosition(int32 InMarkerIndex, float InPosition, const TArray<float>& InMarkerPositions) const
{
	check(bHasValidData);

	const float P_CurrMarker = InMarkerPositions[InMarkerIndex + 0];
	const float P_NextMarker = InMarkerPositions[InMarkerIndex + 1];

	return AreValidMarkerBookendsForPosition(InPosition, P_CurrMarker, P_NextMarker);
}

bool FTimeStretchCurveInstance::AreValidMarkerBookendsForPosition(float InPosition, float InP_CurrMarker, float InP_NextMarker) const
{
	const bool bCurrIsValid = (InPosition >= InP_CurrMarker);
	const bool bNextIsValid = (InPosition <= InP_NextMarker);
	return bCurrIsValid && bNextIsValid;
}

int32 FTimeStretchCurveInstance::BinarySearchMarkerIndex(float InPosition, const TArray<float>& InMarkerPositions) const
{
	check(bHasValidData);

	const int32 NumMarkers = InMarkerPositions.Num();
	ensure(NumMarkers > 0);

	int32 first = 0;
	int32 last = NumMarkers - 2;
	int32 MarkerIndex = INDEX_NONE;
	float P_CurrMarker = 0.f;
	float P_NextMarker = 0.f;

	while (first <= last)
	{
		MarkerIndex = (last + first) / 2;

		P_CurrMarker = InMarkerPositions[MarkerIndex + 0];
		P_NextMarker = InMarkerPositions[MarkerIndex + 1];

		if (AreValidMarkerBookendsForPosition(InPosition, P_CurrMarker, P_NextMarker))
		{
			return MarkerIndex;
		}
		else if (InPosition > P_NextMarker)
		{
			first = MarkerIndex + 1;
		}
		else if (InPosition < P_CurrMarker)
		{
			last = MarkerIndex - 1;
		}
		else
		{
			ensure(false);
			break;
		}
	}

	// Verify the marker we found is the right one
	ensure(MarkerIndex != INDEX_NONE);
	ensure(AreValidMarkerBookendsForPosition(InPosition, P_CurrMarker, P_NextMarker));

	return MarkerIndex;
}

float FTimeStretchCurveInstance::Convert_P_Original_To_Target(int32 InMarkerIndex, float In_P_Original) const
{
	check(bHasValidData);

	const float P_CurrMarker_Original = P_Marker_Original[InMarkerIndex + 0];
	const float P_NextMarker_Original = P_Marker_Original[InMarkerIndex + 1];

	// Note: (P_NextMarker_Original - P_CurrMarker_Original) has been verified to be non zero in 'ConditionallyUpdateTimeStretchCurveCachedData'
	const float Alpha = (In_P_Original - P_CurrMarker_Original) / (P_NextMarker_Original - P_CurrMarker_Original);

	ensure(Alpha >= 0.f);
	ensure(Alpha <= 1.f);

	const float P_CurrMarker_Target = P_Marker_Target[InMarkerIndex + 0];
	const float P_NextMarker_Target = P_Marker_Target[InMarkerIndex + 1];

	const float Out_P_Target = P_CurrMarker_Target + Alpha * (P_NextMarker_Target - P_CurrMarker_Target);
	return Out_P_Target;
}

float FTimeStretchCurveInstance::Convert_P_Target_To_Original(int32 InMarkerIndex, float In_P_Target) const
{
	check(bHasValidData);

	const float P_CurrMarker_Target = P_Marker_Target[InMarkerIndex + 0];
	const float P_NextMarker_Target = P_Marker_Target[InMarkerIndex + 1];

	// Note: (P_NextMarker_Target - P_CurrMarker_Target) has been verified to be non zero in 'ConditionallyUpdateTimeStretchCurveCachedData'
	const float Alpha = (In_P_Target - P_CurrMarker_Target) / (P_NextMarker_Target - P_CurrMarker_Target);

	ensure(Alpha >= 0.f);
	ensure(Alpha <= 1.f);

	const float P_CurrMarker_Original = P_Marker_Original[InMarkerIndex + 0];
	const float P_NextMarker_Original = P_Marker_Original[InMarkerIndex + 1];

	const float Out_P_Original = P_CurrMarker_Original + Alpha * (P_NextMarker_Original - P_CurrMarker_Original);
	return Out_P_Original;
}

float FTimeStretchCurveInstance::Clamp_P_Target(float In_P_Target) const
{
	return FMath::Clamp(In_P_Target, 0.f, P_Marker_Target.Last());
}
