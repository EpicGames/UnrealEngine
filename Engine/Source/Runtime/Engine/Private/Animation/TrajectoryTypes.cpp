// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/TrajectoryTypes.h"

#include "Animation/AnimInstanceProxy.h"
#include "VisualLogger/VisualLogger.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TrajectoryTypes)

FTransformTrajectorySample FTransformTrajectorySample::Lerp(const FTransformTrajectorySample& Other, float Alpha) const
{
	check(Facing.IsNormalized());
	check(Other.Facing.IsNormalized());

	FTransformTrajectorySample Result;
	
	Result.Facing = FQuat::FastLerp(Facing, Other.Facing, Alpha).GetNormalized();
	Result.Position = FMath::Lerp(Position, Other.Position, Alpha);
	Result.TimeInSeconds = FMath::Lerp(TimeInSeconds, Other.TimeInSeconds, Alpha);

	return Result;
}

void FTransformTrajectorySample::SetTransform(const FTransform& Transform)
{
	Position = Transform.GetTranslation();
	Facing = Transform.GetRotation();
}

FArchive& operator<<(FArchive& Ar, FTransformTrajectorySample& Sample)
{
	Ar << Sample.Facing;
	Ar << Sample.Position;
	Ar << Sample.TimeInSeconds;
	return Ar;
}

FTransformTrajectorySample FTransformTrajectory::GetSampleAtTime(float Time, bool bExtrapolate) const
{
	const int32 Num = Samples.Num();
	if (Num > 1)
	{
		const int32 LowerBoundIdx = Algo::LowerBound(Samples, Time, [](const FTransformTrajectorySample& TrajectorySample, float Value)
			{
				return Value > TrajectorySample.TimeInSeconds;
			});

		const int32 NextIdx = FMath::Clamp(LowerBoundIdx, 1, Samples.Num() - 1);
		const int32 PrevIdx = NextIdx - 1;

		const float Denominator = Samples[NextIdx].TimeInSeconds - Samples[PrevIdx].TimeInSeconds;
		if (!FMath::IsNearlyZero(Denominator))
		{
			const float Numerator = Time - Samples[PrevIdx].TimeInSeconds;
			const float LerpValue = bExtrapolate ? Numerator / Denominator : FMath::Clamp(Numerator / Denominator, 0.f, 1.f);
			return Samples[PrevIdx].Lerp(Samples[NextIdx], LerpValue);
		}

		return Samples[PrevIdx];
	}

	if (Num > 0)
	{
		return Samples[0];
	}

	return FTransformTrajectorySample();
}

void UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(UPARAM(ref) const FTransformTrajectory& Trajectory, const UWorld* World, const float DebugThickness, float HeightOffset)
{
#if ENABLE_DRAW_DEBUG
	const FVector OffsetVector = FVector::UpVector * HeightOffset;

	const int32 LastIndex = Trajectory.Samples.Num() - 1;
	if (LastIndex >= 0)
	{
		for (int32 Index = 0; ; ++Index)
		{
			const FVector Pos = Trajectory.Samples[Index].Position + OffsetVector;

			DrawDebugSphere(World, Pos, 1.f, 4, FColor::Black, false, -1.f, SDPG_Foreground, DebugThickness);

			const FRotationMatrix R(FRotator(Trajectory.Samples[Index].Facing));
			const FVector X = R.GetScaledAxis( EAxis::X );
			const FVector Y = R.GetScaledAxis( EAxis::Y );

			const float Scale = 12.f;

			const bool IsPast = Trajectory.Samples[Index].TimeInSeconds <= 0.f;

			DrawDebugLine(World, Pos, Pos + X * Scale, IsPast ? FColor::Red : FColor::Blue, false, -1.f, SDPG_Foreground, DebugThickness);
			DrawDebugLine(World, Pos, Pos + Y * Scale, IsPast ? FColor::Orange : FColor::Turquoise, false, -1.f, SDPG_Foreground, DebugThickness);

			if (Index == LastIndex)
			{
				break;
			}
			
			const FVector NextPos = Trajectory.Samples[Index + 1].Position + OffsetVector;
			DrawDebugLine(World, Pos, NextPos, FColor::Black, false, -1.f, SDPG_Foreground, DebugThickness);
		}
	}
#endif
}

#if ENABLE_ANIM_DEBUG

void UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(const FTransformTrajectory& Trajectory, FAnimInstanceProxy& AnimInstanceProxy, const float DebugThickness, float HeightOffset, int MaxHistorySamples, int MaxPredictionSamples)
{
	const FVector OffsetVector = FVector::UpVector * HeightOffset;
	
	int AvailableHistorySamplesCount = 0;
	
	if (MaxHistorySamples != -1 || MaxPredictionSamples != -1)
	{
		for (int32 i = 0; i < Trajectory.Samples.Num(); ++i)
		{
			if (Trajectory.Samples[i].TimeInSeconds <= 0)
			{
				++AvailableHistorySamplesCount;
			}
			else
			{
				break;
			}
		}
	}
	
	const int32 LastIndex = Trajectory.Samples.Num() - 1;
	const int32 ZeroTimeIndex = AvailableHistorySamplesCount - 1;
	const int32 StartIndex = MaxHistorySamples < 0 ? 0 : FMath::Max(AvailableHistorySamplesCount - MaxHistorySamples, 0); 
	const int32 EndIndex = MaxPredictionSamples < 0 ? LastIndex : FMath::Min(ZeroTimeIndex + MaxPredictionSamples, LastIndex);
	
	if (LastIndex >= 0 && StartIndex <= EndIndex)
	{
		for (int32 Index = StartIndex; ; ++Index)
		{
			const FVector Pos = Trajectory.Samples[Index].Position + OffsetVector;

			AnimInstanceProxy.AnimDrawDebugSphere(Trajectory.Samples[Index].Position + OffsetVector, 1.f, 4, FColor::Black, false, -1.f, DebugThickness, SDPG_Foreground);

			const FRotationMatrix R(FRotator(Trajectory.Samples[Index].Facing));
			const FVector X = R.GetScaledAxis( EAxis::X );
			const FVector Y = R.GetScaledAxis( EAxis::Y );

			const float Scale = 12.f;

			const bool IsPast = Trajectory.Samples[Index].TimeInSeconds <= 0.f;

			AnimInstanceProxy.AnimDrawDebugLine(Pos, Pos + X * Scale, IsPast ? FColor::Red : FColor::Blue, false, -1.f, DebugThickness, SDPG_Foreground);
			AnimInstanceProxy.AnimDrawDebugLine(Pos, Pos + Y * Scale, IsPast ? FColor::Orange : FColor::Turquoise, false, -1.f, DebugThickness, SDPG_Foreground);

			if (Index == EndIndex)
			{
				break;
			}
			
			const FVector NextPos = Trajectory.Samples[Index + 1].Position + OffsetVector;
			AnimInstanceProxy.AnimDrawDebugLine(Pos, NextPos, FColor::Black, false, -1.f, DebugThickness, SDPG_Foreground);
		}
	}
}

void UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(const FTransformTrajectory& Trajectory, const UObject* Owner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const float DebugThickness, float HeightOffset, bool bDisplayVelocity)
{
#if ENABLE_VISUAL_LOG

	UWorld* World = nullptr;
	FVisualLogEntry* CurrentEntry = nullptr;
	if (FVisualLogger::CheckVisualLogInputInternal(Owner, Category.GetCategoryName(), Verbosity, &World, &CurrentEntry) == false)
	{
		return;
	}

	const FVector OffsetVector = FVector::UpVector * HeightOffset;

	const int32 LastIndex = Trajectory.Samples.Num() - 1;
	if (LastIndex >= 0)
	{
		for (int32 Index = 0; ; ++Index)
		{
			const FVector Pos = Trajectory.Samples[Index].Position + OffsetVector;

			CurrentEntry->AddSphere(Pos, 1, Category.GetCategoryName(), Verbosity, FColor::Black, TEXT(""), true);

			const FRotationMatrix R(FRotator(Trajectory.Samples[Index].Facing));
			const FVector X = R.GetScaledAxis(EAxis::X);
			const FVector Y = R.GetScaledAxis(EAxis::Y);

			const float Scale = 12.f;

			const bool IsPast = Trajectory.Samples[Index].TimeInSeconds <= 0.f;

			FVector Velocity = FVector::ZeroVector;
			FString VelocityDebugStr;
			if (Index > 0 && bDisplayVelocity)
			{
				float DeltaTime = Trajectory.Samples[Index].TimeInSeconds - Trajectory.Samples[Index - 1].TimeInSeconds;
				if (DeltaTime > 0.f)
				{
					Velocity = (Trajectory.Samples[Index].Position - Trajectory.Samples[Index - 1].Position) / DeltaTime;
					VelocityDebugStr = FString::Printf(TEXT("Velocity: %.3f"), Velocity.Size());
				}
			}

			CurrentEntry->AddSegment(Pos, Pos + X * Scale, Category.GetCategoryName(), Verbosity, IsPast ? FColor::Red : FColor::Blue, VelocityDebugStr, DebugThickness);
			CurrentEntry->AddSegment(Pos, Pos + Y * Scale, Category.GetCategoryName(), Verbosity, IsPast ? FColor::Orange : FColor::Turquoise, TEXT(""), DebugThickness);

			if (Index == LastIndex)
			{
				break;
			}

			const FVector NextPos = Trajectory.Samples[Index + 1].Position + OffsetVector;
			CurrentEntry->AddSegment(Pos, NextPos, Category.GetCategoryName(), Verbosity, FColor::Black, TEXT(""), DebugThickness);
		}
	}
#endif // ENABLE_VISUAL_LOG
}

#endif // ENABLE_ANIM_DEBUG

FArchive& operator<<(FArchive& Ar, FTransformTrajectory& Trajectory)
{
	Ar << Trajectory.Samples;
	return Ar;
}
