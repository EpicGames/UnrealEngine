// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/VariableFrameStrippingSettings.h"
#include "Interfaces/ITargetPlatform.h"
#include "Animation/AnimationSettings.h"
#include "AnimationUtils.h"
#include "AnimationCompression.h"
#include "PlatformInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VariableFrameStrippingSettings)

UVariableFrameStrippingSettings::UVariableFrameStrippingSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UseVariableFrameStripping = FPerPlatformBool(false);//
	FrameStrippingRate = 3;//default Value Set to 3 as 1 is no stripping and 2 is default stripping 3 is the lowest number with an additional effect. 
}

#if WITH_EDITORONLY_DATA
/** Generates a DDC key that takes into account the current settings, selected codec, input anim sequence and TargetPlatform */
void UVariableFrameStrippingSettings::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	const ITargetPlatform* TargetPlatform = KeyArgs.TargetPlatform;
	if (TargetPlatform != nullptr)
	{
		FName TargetPlatformName = TargetPlatform->GetTargetPlatformInfo().IniPlatformName;
		bool PerPlatformUse = UseVariableFrameStripping.GetValueForPlatform(TargetPlatformName);
		Ar << PerPlatformUse;
		int32 PerPlatformRate = FrameStrippingRate.GetValueForPlatform(TargetPlatformName);
		Ar << PerPlatformRate;
	}
}
#endif
