// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimationAsset.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AnimMontage.h"
#include "Engine/AssetUserData.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AssetMappingTable.h"
#include "Animation/AnimMetaData.h"
#include "Animation/AnimSequence.h"
#include "AnimationUtils.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "Animation/BlendSpace.h"
#include "Animation/PoseAsset.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimationSequenceCompiler.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationAsset)

#define LOCTEXT_NAMESPACE "AnimationAsset"

#define LEADERSCORE_FOLLOWER_OFFSET	-2.f // offset to score any follower after any possible leader
#define LEADERSCORE_ALWAYSLEADER  	2.f
#define LEADERSCORE_MONTAGE			3.f

FVector FRootMotionMovementParams::RootMotionScale(1.0f, 1.0f, 1.0f);
const TArray<FName> FMarkerTickContext::DefaultMarkerNames;

//////////////////////////////////////////////////////////////////////////
// FAnimGroupInstance

void FAnimGroupInstance::TestTickRecordForLeadership(EAnimGroupRole::Type MembershipType)
{
	check(ActivePlayers.Num() > 0);

	// Always set leader score if you have potential to be leader
	// that way if the top leader fails, we'll continue to search next available leader.
	const int32 TestIndex = ActivePlayers.Num() - 1;
	FAnimTickRecord& Candidate = ActivePlayers[TestIndex];

	// Handle Montage candidate.
	if (Candidate.SourceAsset->IsA<UAnimMontage>())
	{
		// Check if the candidate has a higher weight.
		if (Candidate.EffectiveBlendWeight > MontageLeaderWeight)
		{
			// If this is going to be leader, clean ActivePlayers because we don't sync multi montages.
			const int32 LastIndex = TestIndex - 1;
			if (LastIndex >= 0)
			{
				// Removing based on the last index works because any montage's tick records are all added during Montage Update which happens before the Anim Graph is updated.
				ActivePlayers.RemoveAt(LastIndex, 1);
			}

			// At this time, it should only have one.
			check(ActivePlayers.Num() == 1);

			// then override
			// @note : leader weight doesn't applied WITHIN montages
			// we still only contain one montage at a time, if this montage fails, next candidate will get the chance, not next weight montage
			MontageLeaderWeight = Candidate.EffectiveBlendWeight;
			Candidate.LeaderScore = LEADERSCORE_MONTAGE;
		}
		else
		{
			if (TestIndex != 0)
			{
				// We delete the later ones because we only have one montage for leader 
				// this can happen if there was already active one with higher weight. 
				ActivePlayers.RemoveAt(TestIndex, 1);
			}
		}

		ensureAlways(ActivePlayers.Num() == 1);
	}
	// Handle Sequence or BlendSpace candidate.
	else
	{
		switch (MembershipType)
		{
		case EAnimGroupRole::CanBeLeader:
		case EAnimGroupRole::TransitionLeader:
			Candidate.LeaderScore = Candidate.EffectiveBlendWeight; // LeaderScore within [0, 1] interval
			break;
		case EAnimGroupRole::AlwaysLeader:
		case EAnimGroupRole::ExclusiveAlwaysLeader:
			// Always set the leader index
			Candidate.LeaderScore = LEADERSCORE_ALWAYSLEADER;  // LeaderScore within [2, 2] interval
			break;
		default:
		case EAnimGroupRole::AlwaysFollower:
		case EAnimGroupRole::TransitionFollower:
			// making sure that the LeaderScore of any follower is always less than any possible leader, 
			// so leaders happen to be sorted before any follower
			// in case there's no leader the follower with highest LeaderScore will be elected as anim group leader
			Candidate.LeaderScore = LEADERSCORE_FOLLOWER_OFFSET + Candidate.EffectiveBlendWeight; // LeaderScore within [-2, -1] interval
			break;
		}
	}
}

void FAnimGroupInstance::Finalize(const FAnimGroupInstance* PreviousGroup)
{
	// Reset follower records if the previous group is non-existent, the group leader changed, or the montage leader disappears
	if (!PreviousGroup || PreviousGroup->GroupLeaderIndex != GroupLeaderIndex || (PreviousGroup->MontageLeaderWeight > 0.f && MontageLeaderWeight == 0.f))
	{
		UE_LOGF(LogAnimMarkerSync, Log, "Resetting Marker Sync Groups");

		for (int32 RecordIndex = GroupLeaderIndex + 1; RecordIndex < ActivePlayers.Num(); ++RecordIndex)
		{
			ActivePlayers[RecordIndex].MarkerTickRecord->Reset();
		}
	}
}

void FAnimGroupInstance::Prepare(const FAnimGroupInstance* PreviousGroup)
{
	// Sort asset players by leader score.
	ActivePlayers.Sort();

	// Prepare group, if leader has any markers.
	const TArray<FName>* MarkerNames = ActivePlayers[0].SourceAsset->GetUniqueMarkerNames();
	const bool bLeaderHasMarkers = MarkerNames && !MarkerNames->IsEmpty();
	
	if (bLeaderHasMarkers)
	{
		// Get leader's markers.
		ValidMarkers = *MarkerNames;

		// Enable marker based syncing for leader and instance group.
		ActivePlayers[0].bCanUseMarkerSync = true;
		bCanUseMarkerSync = true;

		// Prepare asset player candidates.
		for (int32 ActivePlayerIndex = 0; ActivePlayerIndex < ActivePlayers.Num(); ++ActivePlayerIndex)
		{
			FAnimTickRecord& Candidate = ActivePlayers[ActivePlayerIndex];

			// Reset candidate's marker tick record if needed.
			if (PreviousGroup)
			{
				bool bCandidateFound = false;
				
				for (const FAnimTickRecord& PrevRecord : PreviousGroup->ActivePlayers)
				{
					if (PrevRecord.MarkerTickRecord == Candidate.MarkerTickRecord)
					{
						// Found previous record for "us"
						if (PrevRecord.SourceAsset != Candidate.SourceAsset)
						{
							Candidate.MarkerTickRecord->Reset(); // Changed animation, clear our cached data.
						}
						bCandidateFound = true;
						break;
					}
				}
				
				if (!bCandidateFound)
				{
					Candidate.MarkerTickRecord->Reset(); // We weren't active last frame, invalidate record.
				}
			}

			// Filter follower's markers that are not shared in common with the group's candidate leader.
			if (ActivePlayerIndex != 0 && ValidMarkers.Num() > 0)
			{
				// Let anims with no markers use length scaling sync.
				const TArray<FName>* PlayerMarkerNames = Candidate.SourceAsset->GetUniqueMarkerNames();
				const bool bFollowerHasMarkers = PlayerMarkerNames && !PlayerMarkerNames->IsEmpty();
				
				if (bFollowerHasMarkers) 
				{
					// Make follower use marker based-syncing.
					Candidate.bCanUseMarkerSync = true;

					// Filter.
					for (int32 ValidMarkerIndex = ValidMarkers.Num() - 1; ValidMarkerIndex >= 0; --ValidMarkerIndex)
					{
						FName& MarkerName = ValidMarkers[ValidMarkerIndex];
						
						if (!PlayerMarkerNames->Contains(MarkerName))
						{
							ValidMarkers.RemoveAtSwap(ValidMarkerIndex, EAllowShrinking::No);
						}
					}
				}
			}
		}

		// Ensure group can use maker based syncing.
		bCanUseMarkerSync = ValidMarkers.Num() > 0;
		
		// Alphabetical ordering for markers.
		ValidMarkers.Sort(FNameLexicalLess());

		// Source marker data changed.
		if (!PreviousGroup || (ValidMarkers != PreviousGroup->ValidMarkers))
		{
			for (int32 InternalActivePlayerIndex = 0; InternalActivePlayerIndex < ActivePlayers.Num(); ++InternalActivePlayerIndex)
			{
				ActivePlayers[InternalActivePlayerIndex].MarkerTickRecord->Reset();
			}
		}
	}
	
	// Leader has no markers or all them were filtered out, fallback to length based syncing.
	if (!bLeaderHasMarkers || !bCanUseMarkerSync)
	{
		// We can't use sync markers in sync group.
		bCanUseMarkerSync = false;

		// Invalidate markers.
		ValidMarkers.Reset();

		// Ensure tick records do not use marker-based syncing.
		for (FAnimTickRecord& AnimTickRecord : ActivePlayers)
		{
			AnimTickRecord.MarkerTickRecord->Reset();
			AnimTickRecord.bCanUseMarkerSync = false;
		}
	}
}

void FAnimTickRecord::AllocateContextDataContainer()
{
	ContextData = MakeShared<TArray<TUniquePtr<const UE::Anim::IAnimNotifyEventContextDataInterface>>>();
}

FAnimTickRecord::FAnimTickRecord(UAnimSequenceBase* InSequence, bool bInLooping, float InPlayRate, bool bInIsEvaluator, float InFinalBlendWeight, float& InCurrentTime, FMarkerTickRecord& InMarkerTickRecord)
{
	SourceAsset = InSequence;
	TimeAccumulator = &InCurrentTime;
	MarkerTickRecord = &InMarkerTickRecord;
	PlayRateMultiplier = InPlayRate;
	EffectiveBlendWeight = InFinalBlendWeight;
	bLooping = bInLooping;
	bIsEvaluator = bInIsEvaluator;
}

FAnimTickRecord::FAnimTickRecord(
	UBlendSpace* InBlendSpace, const FVector& InBlendInput, TArray<FBlendSampleData>& InBlendSampleDataCache, FBlendFilter& InBlendFilter, bool bInLooping, 
	float InPlayRate, bool bTeleportToTime, bool bInIsEvaluator, float InFinalBlendWeight, float& InCurrentTime, FMarkerTickRecord& InMarkerTickRecord)
{
	SourceAsset = InBlendSpace;
	BlendSpace.BlendSpacePositionX = InBlendInput.X;
	BlendSpace.BlendSpacePositionY = InBlendInput.Y;
	BlendSpace.BlendSampleDataCache = &InBlendSampleDataCache;
	BlendSpace.BlendFilter = &InBlendFilter;
	BlendSpace.bTeleportToTime = bTeleportToTime;
	TimeAccumulator = &InCurrentTime;
	MarkerTickRecord = &InMarkerTickRecord;
	PlayRateMultiplier = InPlayRate;
	EffectiveBlendWeight = InFinalBlendWeight;
	bLooping = bInLooping;
	bIsEvaluator = bInIsEvaluator;
}

FAnimTickRecord::FAnimTickRecord(UAnimMontage* InMontage, float InCurrentPosition, float InWeight, TArray<FPassedMarker>& InMarkersPassedThisTick, FMarkerTickRecord& InMarkerTickRecord)
{
	SourceAsset = InMontage;
	Montage.CurrentPosition = InCurrentPosition;
	Montage.MarkersPassedThisTick = &InMarkersPassedThisTick;
	MarkerTickRecord = &InMarkerTickRecord;
	PlayRateMultiplier = 1.f; // we don't care here, this is alreayd applied in the montageinstance::Advance
	EffectiveBlendWeight = InWeight;
	bLooping = false;
}

FAnimTickRecord::FAnimTickRecord(UPoseAsset* InPoseAsset, float InFinalBlendWeight)
{
	SourceAsset = InPoseAsset;
	EffectiveBlendWeight = InFinalBlendWeight;
}

void FAnimTickRecord::GatherContextData(const FAnimationUpdateContext& InContext)
{
	bActiveContext = InContext.IsActive();
	if(InContext.GetSharedContext())
	{
		TArray<TUniquePtr<const UE::Anim::IAnimNotifyEventContextDataInterface>> NewContextData;
		InContext.GetSharedContext()->MessageStack.MakeEventContextData(NewContextData);
		if(NewContextData.Num())
		{
			if (!ContextData.IsValid())
			{
				AllocateContextDataContainer();
			}

			ContextData->Append(MoveTemp(NewContextData));
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// UAnimationAsset

void UAnimationAsset::PostLoad()
{
	LLM_SCOPE(ELLMTag::Animation);

	Super::PostLoad();

	// Load skeleton, to make sure anything accessing from PostLoad
	// skeleton is ready
	if (Skeleton)
	{
		Skeleton->ConditionalPreload();
		Skeleton->ConditionalPostLoad();
	}

#if WITH_EDITORONLY_DATA
	// Load Parent Asset, to make sure anything accessing from PostLoad has valid data to access
	if (ParentAsset)
	{
		ParentAsset->ConditionalPreload();
		ParentAsset->ConditionalPostLoad();
	}
#endif

	ValidateSkeleton();

	check( Skeleton==NULL || SkeletonGuid.IsValid() );

#if WITH_EDITOR
	UpdateParentAsset();
#endif // WITH_EDITOR
}

void UAnimationAsset::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::Animation);

	Super::Serialize(Ar);

	if (Ar.UEVer() >= VER_UE4_SKELETON_GUID_SERIALIZATION)
	{
		Ar << SkeletonGuid;
	}
}

UAnimMetaData* UAnimationAsset::FindMetaDataByClass(const TSubclassOf<UAnimMetaData> MetaDataClass) const
{
	UAnimMetaData* FoundMetaData = nullptr;

	if (UClass* TargetClass = MetaDataClass.Get())
	{
		for (UAnimMetaData* MetaDataInstance : MetaData)
		{
			if (MetaDataInstance && MetaDataInstance->IsA(TargetClass))
			{
				FoundMetaData = MetaDataInstance;
				break;
			}
		}
	}

	return FoundMetaData;
}

void UAnimationAsset::AddMetaData(UAnimMetaData* MetaDataInstance)
{
	MetaData.Add(MetaDataInstance);
}

void UAnimationAsset::RemoveMetaData(UAnimMetaData* MetaDataInstance)
{
	MetaData.Remove(MetaDataInstance);
}

void UAnimationAsset::RemoveMetaData(TArrayView<UAnimMetaData*> MetaDataInstances)
{
	MetaData.RemoveAll(
		[&](UAnimMetaData* MetaDataInstance)
	{
		return MetaDataInstances.Contains(MetaDataInstance);
	});
}

void UAnimationAsset::SetSkeleton(USkeleton* NewSkeleton)
{
#if WITH_EDITOR
	OnSetSkeleton(NewSkeleton);
#endif // WITH_EDITOR
	Skeleton = NewSkeleton;
	if (Skeleton)
	{
		SkeletonGuid = NewSkeleton->GetGuid();
	}
	else
	{
		SkeletonGuid.Invalidate();
	}
}

USkeletalMesh* UAnimationAsset::GetPreviewMesh(bool bFindIfNotSet)
{
#if WITH_EDITORONLY_DATA
	USkeletalMesh* PreviewMesh = PreviewSkeletalMesh.LoadSynchronous();
	// if somehow skeleton changes, just nullify it. 
	if (PreviewMesh && (PreviewMesh->GetSkeleton() == nullptr || !PreviewMesh->GetSkeleton()->IsCompatibleForEditor(Skeleton)))
	{
		PreviewMesh = nullptr;
		SetPreviewMesh(nullptr);
	}

	return PreviewMesh;
#else
	return nullptr;
#endif
}

USkeletalMesh* UAnimationAsset::GetPreviewMesh() const
{
#if WITH_EDITORONLY_DATA
	if (!PreviewSkeletalMesh.IsValid())
	{
		PreviewSkeletalMesh.LoadSynchronous();
	}
	return PreviewSkeletalMesh.Get();
#else
	return nullptr;
#endif
}

void UAnimationAsset::SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty/*=true*/)
{
#if WITH_EDITORONLY_DATA
	if(bMarkAsDirty)
	{
		Modify();
	}
	PreviewSkeletalMesh = PreviewMesh;
#endif
}

#if WITH_EDITOR
void UAnimationAsset::RemapTracksToNewSkeleton(USkeleton* NewSkeleton, bool bConvertSpaces)
{
	SetSkeleton(NewSkeleton);
}

bool UAnimationAsset::ReplaceSkeleton(USkeleton* NewSkeleton, bool bConvertSpaces/*=false*/)
{
	// if it's not same 
	if (NewSkeleton && (NewSkeleton != Skeleton || NewSkeleton->GetGuid() != SkeletonGuid))
	{
		OnSetSkeleton(NewSkeleton);

		// get all sequences that need to change
		TArray<UAnimationAsset*> AnimAssetsToReplace;

		// Always add 'this' asset
		AnimAssetsToReplace.Add(this);

		// We dont use the return value from this function because we always add 'this' above
		GetAllAnimationSequencesReferred(AnimAssetsToReplace);

		TArray<UAnimSequence*> Sequences;

		//Firstly need to remap
		for (UAnimationAsset* AnimAsset : AnimAssetsToReplace)
		{
			//Make sure animation has finished loading before we start messing with it
			AnimAsset->ConditionalPreload();
			AnimAsset->ConditionalPostLoad();

			if (AnimAsset->GetSkeleton() != GetSkeleton())
			{
				UE_LOGF(LogAnimation, Warning, "AnimationAsset referencing asset using different skeleton. This will generate undeterministic builds and may cause additional warnings from undeterministic ordering of skeleton conforming. Please Fix the Asset : AnimationAsset: [%ls] (Skeleton: [%ls]) - ReferencedAsset : [%ls] (Skeleton: [%ls]) (Conforming Skeleton from [%ls] to [%ls]",
					*GetPathName(),
					*GetSkeleton()->GetPathName(),
					*AnimAsset->GetPathName(),
					*AnimAsset->GetSkeleton()->GetPathName(),
					*AnimAsset->GetSkeleton()->GetPathName(),
					*NewSkeleton->GetPathName());
			}

			// This ensure that in subsequent behaviour the RawData GUID is never 'new-ed' but always calculated from the 
			// raw animation data itself.
			if (UAnimSequence* Sequence = Cast<UAnimSequence>(AnimAsset))
			{
				Sequences.Add(Sequence);
			}
			else
			{
				// these two are different functions for now
				// technically if you have implementation for Remap, it will also set skeleton 
				AnimAsset->RemapTracksToNewSkeleton(NewSkeleton, bConvertSpaces);
			}
		}

		UE::Anim::FAnimSequenceCompilingManager::Get().FinishCompilation(Sequences);
		for (UAnimSequence* Sequence : Sequences)
		{
			Sequence->GetController().OpenBracket(LOCTEXT("ReplaceSkeleton_Bracket", "Replacing USkeleton"));
			Sequence->RemapTracksToNewSkeleton(NewSkeleton, bConvertSpaces);
		}

		//Second need to process anim sequences themselves. This is done in two stages as additives can rely on other animations.
		for (UAnimSequence* Sequence : Sequences)
		{
			Sequence->GetController().CloseBracket();
		}

		return true;
	}

	return false;
}

bool UAnimationAsset::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationSequences, bool bRecursive /*= true*/) 
{
	//@todo:@fixme: this doesn't work for retargeting because postload gets called after duplication, mixing up the mapping table
	// because skeleton changes, for now we don't support retargeting for parent asset, it will disconnect, and just duplicate everything else
// 	if (ParentAsset)
// 	{
// 		ParentAsset->HandleAnimReferenceCollection(AnimationSequences, bRecursive);
// 	}
// 
// 	if (AssetMappingTable)
// 	{
// 		AssetMappingTable->GetAllAnimationSequencesReferred(AnimationSequences, bRecursive);
// 	}

	return (AnimationSequences.Num() > 0);
}

void UAnimationAsset::HandleAnimReferenceCollection(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive)
{
	AnimationAssets.AddUnique(this);
	if (bRecursive)
	{
		// anim sequence still should call this. since bRecursive is true, I'm not sending the parameter with it
		GetAllAnimationSequencesReferred(AnimationAssets);
	}
}

void UAnimationAsset::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap)
{
	//@todo:@fixme: this doesn't work for retargeting because postload gets called after duplication, mixing up the mapping table
	// because skeleton changes, for now we don't support retargeting for parent asset, it will disconnect, and just duplicate everything else
	if (ParentAsset)
	{
		// clear up, so that it doesn't try to use from other asset
		ParentAsset = nullptr;
		AssetMappingTable = nullptr;
	}

// 	if (ParentAsset)
// 	{
// 		// now fix everythign else
// 		UAnimationAsset* const* ReplacementAsset = ReplacementMap.Find(ParentAsset);
// 		if (ReplacementAsset)
// 		{
// 			ParentAsset = *ReplacementAsset;
// 			ParentAsset->ReplaceReferredAnimations(ReplacementMap);
// 		}
// 	}
// 
// 	if (AssetMappingTable)
// 	{
// 		AssetMappingTable->ReplaceReferredAnimations(ReplacementMap);
// 	}
}

void UAnimationAsset::UpdateParentAsset()
{
	ValidateParentAsset();

	if (ParentAsset)
	{
		TArray<UAnimationAsset*> AnimAssetsReferencedDirectly;
		if (ParentAsset->GetAllAnimationSequencesReferred(AnimAssetsReferencedDirectly, false))
		{
			AssetMappingTable->RefreshAssetList(AnimAssetsReferencedDirectly);
		}
	}
	else
	{
		// if somehow source data is gone, there is nothing much to do here
		ParentAsset = nullptr;
		AssetMappingTable = nullptr;
	}

	if (ParentAsset)
	{
		RefreshParentAssetData();
	}
}

void UAnimationAsset::ValidateParentAsset()
{
	if (ParentAsset)
	{
		if (ParentAsset->GetSkeleton() != GetSkeleton())
		{
			// parent asset changed skeleton, so we'll have to discard parent asset
			UE_LOGF(LogAnimation, Warning, "%ls: ParentAsset %ls linked to different skeleton. Removing the reference.", *GetName(), *GetNameSafe(ParentAsset));
			ParentAsset = nullptr;
			Modify();
		}
		else if (ParentAsset->StaticClass() != StaticClass())
		{
			// parent asset changed skeleton, so we'll have to discard parent asset
			UE_LOGF(LogAnimation, Warning, "%ls: ParentAsset %ls class type doesn't match. Removing the reference.", *GetName(), *GetNameSafe(ParentAsset));
			ParentAsset = nullptr;
			Modify();
		}
	}
}

void UAnimationAsset::RefreshParentAssetData()
{
	// should only allow within the same skeleton
	ParentAsset->ChildrenAssets.AddUnique(this);
	MetaData = ParentAsset->MetaData;
	PreviewPoseAsset = ParentAsset->PreviewPoseAsset;
	PreviewSkeletalMesh = ParentAsset->PreviewSkeletalMesh;
}

void UAnimationAsset::SetParentAsset(UAnimationAsset* InParentAsset)
{
	// only same class and same skeleton
	if (InParentAsset && InParentAsset->HasParentAsset() == false && 
		InParentAsset->StaticClass() == StaticClass() && InParentAsset->GetSkeleton() == GetSkeleton())
	{
		ParentAsset = InParentAsset;

		// if ParentAsset exists, just create mapping table.
		// it becomes messy if we only created when we have assets to map
		if (AssetMappingTable == nullptr)
		{
			AssetMappingTable = NewObject<UAssetMappingTable>(this);
		}
		else
		{
			AssetMappingTable->Clear();
		}

		UpdateParentAsset();
	}
	else
	{
		// otherwise, clear it
		ParentAsset = nullptr;
		AssetMappingTable = nullptr;
	}
}

bool UAnimationAsset::RemapAsset(UAnimationAsset* SourceAsset, UAnimationAsset* TargetAsset)
{
	if (AssetMappingTable)
	{
		if (AssetMappingTable->RemapAsset(SourceAsset, TargetAsset))
		{
			RefreshParentAssetData();
			return true;
		}
	}

	return false;
}
#endif

void UAnimationAsset::ValidateSkeleton()
{
	if (Skeleton && Skeleton->GetGuid() != SkeletonGuid)
	{
#if WITH_EDITOR
		// reset Skeleton
		ReplaceSkeleton(Skeleton);
		UE_LOGF(LogAnimation, Verbose, "Needed to reset skeleton. Resave this asset to speed up load time: %ls", *GetPathNameSafe(this));
#else
		UE_LOGF(LogAnimation, Warning, "Skeleton GUID is out-of-date, this should have been updated during cook. %ls", *GetPathNameSafe(this));
#endif
	}
}

void UAnimationAsset::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != NULL)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != NULL)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UAnimationAsset::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

void UAnimationAsset::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* UAnimationAsset::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

#if WITH_EDITOR
void UAnimationAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	for (UAssetUserData* Datum : AssetUserData)
	{
		if (Datum != nullptr)
		{
			Datum->PostEditChangeOwner(PropertyChangedEvent);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimationAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	
	for (const UAssetUserData* UserData : AssetUserData)
	{
		if (UserData)
		{
			UserData->GetAssetRegistryTags(Context);
		}
	}
	
	Context.AddTag( FAssetRegistryTag("HasParentAsset", HasParentAsset() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Hidden) );
}

EDataValidationResult UAnimationAsset::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = UObject::IsDataValid(Context);
	for (const UAssetUserData* Datum : AssetUserData)
	{
		if(Datum != nullptr && Datum->IsDataValid(Context) == EDataValidationResult::Invalid)
		{
			Result = EDataValidationResult::Invalid;
		}
	}
	for (const UAnimMetaData* Datum : MetaData)
	{
		if (Datum != nullptr)
		{
			Result = CombineDataValidationResults(Result, Datum->IsDataValid(Context));
		}
	}
	return Result;
}
#endif

///////////////////////////////////////////////////////////////////////////////////////
//
// FBlendSampleData
//
////////////////////////////////////////////////////////////////////////////////////////
void FBlendSampleData::NormalizeDataWeight(TArray<FBlendSampleData>& SampleDataList)
{
	float TotalSum = 0.f;

	check(SampleDataList.Num() > 0);
	const int32 NumBones = SampleDataList[0].PerBoneBlendData.Num();

	FMemMark Mark(FMemStack::Get());
	TArray<float, TMemStackAllocator<>> PerBoneTotalSums;
	PerBoneTotalSums.AddZeroed(NumBones);

	for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
	{
		checkf(SampleDataList[PoseIndex].PerBoneBlendData.Num() == NumBones, TEXT("Attempted to normalise a blend sample list, but the samples have differing numbers of bones."));

		TotalSum += SampleDataList[PoseIndex].TotalWeight;

		if (SampleDataList[PoseIndex].PerBoneBlendData.Num() > 0)
		{
			// now interpolate the per bone weights
			for (int32 BoneIndex = 0; BoneIndex<NumBones; BoneIndex++)
			{
				PerBoneTotalSums[BoneIndex] += SampleDataList[PoseIndex].PerBoneBlendData[BoneIndex];
			}
		}
	}

	// Re-normalize Pose weight
	if (TotalSum > ZERO_ANIMWEIGHT_THRESH)
	{
		if (FMath::Abs<float>(TotalSum - 1.f) > ZERO_ANIMWEIGHT_THRESH)
		{
			for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
			{
				SampleDataList[PoseIndex].TotalWeight /= TotalSum;
			}
		}
	}
	else
	{
		for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
		{
			SampleDataList[PoseIndex].TotalWeight = 1.0f / SampleDataList.Num();
		}
	}

	// Re-normalize per bone weights.
	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		if (PerBoneTotalSums[BoneIndex] > ZERO_ANIMWEIGHT_THRESH)
		{
			if (FMath::Abs<float>(PerBoneTotalSums[BoneIndex] - 1.f) > ZERO_ANIMWEIGHT_THRESH)
			{
				for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
				{
					SampleDataList[PoseIndex].PerBoneBlendData[BoneIndex] /= PerBoneTotalSums[BoneIndex];
				}
			}
		}
		else
		{
			for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
			{
				SampleDataList[PoseIndex].PerBoneBlendData[BoneIndex] = 1.0f / SampleDataList.Num();
			}
		}
	}
}

void FBlendSampleData::NormalizeDataWeight(const TArrayView<FBlendSampleData*>& SampleDataList)
{
	float TotalSum = 0.f;

	check(SampleDataList.Num() > 0);
	const int32 NumBones = SampleDataList[0]->PerBoneBlendData.Num();

	FMemMark Mark(FMemStack::Get());
	TArray<float, TMemStackAllocator<>> PerBoneTotalSums;
	PerBoneTotalSums.AddZeroed(NumBones);

	for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
	{
		checkf(SampleDataList[PoseIndex]->PerBoneBlendData.Num() == NumBones, TEXT("Attempted to normalise a blend sample list, but the samples have differing numbers of bones."));

		TotalSum += SampleDataList[PoseIndex]->TotalWeight;

		if (SampleDataList[PoseIndex]->PerBoneBlendData.Num() > 0)
		{
			// now interpolate the per bone weights
			for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
			{
				PerBoneTotalSums[BoneIndex] += SampleDataList[PoseIndex]->PerBoneBlendData[BoneIndex];
			}
		}
	}

	// Re-normalize Pose weight
	if (TotalSum > ZERO_ANIMWEIGHT_THRESH)
	{
		if (FMath::Abs<float>(TotalSum - 1.f) > ZERO_ANIMWEIGHT_THRESH)
		{
			for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
			{
				SampleDataList[PoseIndex]->TotalWeight /= TotalSum;
			}
		}
	}
	else
	{
		for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
		{
			SampleDataList[PoseIndex]->TotalWeight = 1.0f / SampleDataList.Num();
		}
	}

	// Re-normalize per bone weights.
	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		if (PerBoneTotalSums[BoneIndex] > ZERO_ANIMWEIGHT_THRESH)
		{
			if (FMath::Abs<float>(PerBoneTotalSums[BoneIndex] - 1.f) > ZERO_ANIMWEIGHT_THRESH)
			{
				for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
				{
					SampleDataList[PoseIndex]->PerBoneBlendData[BoneIndex] /= PerBoneTotalSums[BoneIndex];
				}
			}
		}
		else
		{
			for (int32 PoseIndex = 0; PoseIndex < SampleDataList.Num(); PoseIndex++)
			{
				SampleDataList[PoseIndex]->PerBoneBlendData[BoneIndex] = 1.0f / SampleDataList.Num();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE // "AnimationAsset"
