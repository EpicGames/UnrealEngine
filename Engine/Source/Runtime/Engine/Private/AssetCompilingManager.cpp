// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetCompilingManager.h"
#include "AsyncCompilationHelpers.h"

#include "HAL/LowLevelMemStats.h" // IWYU pragma: keep
#include "HAL/LowLevelMemTracker.h" // IWYU pragma: keep

DECLARE_LLM_MEMORY_STAT(TEXT("AssetCompilation"), STAT_AssetCompilationLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AssetCompilation"), STAT_AssetCompilationSummaryLLM, STATGROUP_LLM);
LLM_DEFINE_TAG(AssetCompilation, NAME_None, NAME_None, GET_STATFNAME(STAT_AssetCompilationLLM), GET_STATFNAME(STAT_AssetCompilationSummaryLLM));

#include "Misc/QueuedThreadPool.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "Misc/CommandLine.h"
#include "ActorDeferredScriptManager.h"
#include "StaticMeshCompiler.h"
#include "TextureCompiler.h"
#include "SoundWaveCompiler.h"
#include "ObjectCacheContext.h"
#include "SkinnedAssetCompiler.h"
#include "Algo/TopologicalSort.h"
#include "Algo/Find.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Animation/AnimationSequenceCompiler.h"
#include "Animation/AnimBankCompiler.h"
#include "Animation/AnimSequenceTransformProviderCompiler.h"
#include "ShaderCompiler.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_EDITOR
#include "DerivedDataBuildSchedulerQueue.h"
#include "DerivedDataThreadPoolTask.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "Logging/StructuredLog.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetCompilingManager)

#define LOCTEXT_NAMESPACE "AssetCompilingManager"

#if WITH_EDITOR

static AsyncCompilationHelpers::FAsyncCompilationStandardCVars CVarAsyncCompilationStandard(
	TEXT("Asset"), 
	TEXT("assets"),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			FAssetCompilingManager::Get().FinishAllCompilation();
		}
	));


// AsyncAssetCompilationMemoryPerCore is actually the value used (in GB)
//	for Required Memory when the Task reports -1 (Unknown)
//	default is 4 GB
static TAutoConsoleVariable<int32> CVarAsyncAssetCompilationMemoryPerCore(
	TEXT("Editor.AsyncAssetCompilationMemoryPerCore"),
	1,
	TEXT("How much memory (in GB) should tasks reserve that report a required memory amount Unknown (-1).\n"),
	ECVF_Default);

// AsyncAssetCompilationMaxMemoryUsage clamps system available memory
static TAutoConsoleVariable<int32> CVarAsyncAssetCompilationMaxMemoryUsage(
	TEXT("Editor.AsyncAssetCompilationMaxMemoryUsage"),
	0,
	TEXT("0 - No hard memory limit, will be tuned against system available memory (recommended default).\n")
	TEXT("N - Try to limit total memory usage for asset compilation to this amount (in GB).\n")
	TEXT("Try to stay under specified memory limit for asset compilation by reducing concurrency when under memory pressure.\n"),
	ECVF_Default);

TRACE_DECLARE_INT_COUNTER(AsyncCompilationMaxConcurrency, TEXT("AsyncCompilation/MaxConcurrency"));
TRACE_DECLARE_INT_COUNTER(AsyncCompilationConcurrency, TEXT("AsyncCompilation/Concurrency"));
TRACE_DECLARE_MEMORY_COUNTER(AsyncCompilationAvailableMemory, TEXT("AsyncCompilation/AvailableMemory"));
TRACE_DECLARE_MEMORY_COUNTER(AsyncCompilationTotalEstimatedMemory, TEXT("AsyncCompilation/TotalEstimatedMemory"));


TRACE_DECLARE_INT_COUNTER(AsyncDelayedTask, TEXT("AsyncCompilation/DelayedTask"));
TRACE_DECLARE_INT_COUNTER(AsyncDelayedTaskActive, TEXT("AsyncCompilation/DelayedTaskActive"));

#endif //#if WITH_EDITOR

namespace AssetCompilingManagerImpl
{
	static void EnsureInitializedCVars()
	{
#if WITH_EDITOR
		static bool bIsInitialized = false;

		if (!bIsInitialized)
		{
			bIsInitialized = true;

			AsyncCompilationHelpers::EnsureInitializedCVars(
				TEXT("asset"),
				CVarAsyncCompilationStandard.AsyncCompilation,
				CVarAsyncCompilationStandard.AsyncCompilationMaxConcurrency
			);
		}
#endif
	}

#if WITH_EDITOR
	using namespace UE::DerivedData;

	class FAssetCompilingManagerMemoryQueue final : public IBuildSchedulerMemoryQueue
	{
		void Reserve(uint64 Memory, IRequestOwner& Owner, TUniqueFunction<void ()>&& OnComplete) final
		{
			LaunchTaskInThreadPool(Memory,TEXT("FAssetCompilingManagerMemoryQueue"), Owner, FAssetCompilingManager::Get().GetThreadPool(), MoveTemp(OnComplete));
		}
	};

	static FAssetCompilingManagerMemoryQueue GMemoryQueue;
#endif
} // AssetCompilingManagerImpl

#if WITH_EDITOR

class FMemoryBoundQueuedThreadPoolWrapper : public FQueuedThreadPoolWrapper
{
public:
	/**
	 * InWrappedQueuedThreadPool  Underlying thread pool to schedule task to.
	 * InMaxConcurrency           Maximum number of concurrent tasks allowed, -1 will limit concurrency to number of threads available in the underlying thread pool.
	 * InPriorityMapper           Thread-safe function used to map any priority from this Queue to the priority that should be used when scheduling the task on the underlying thread pool.
	 */
	FMemoryBoundQueuedThreadPoolWrapper(FQueuedThreadPool* InWrappedQueuedThreadPool, TFunction<int32 ()> InMaxForegroundConcurrency, TFunction<int32()> InMaxBackgroundConcurrency, TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> InPriorityMapper = [](EQueuedWorkPriority InPriority) { return InPriority; })
		: FQueuedThreadPoolWrapper(InWrappedQueuedThreadPool, -1, InPriorityMapper)
		, MemoryInfo([this]() { return GetMemoryAvailableForAssetCompilation(); })
		, MaxForegroundConcurrency(InMaxForegroundConcurrency)
		, MaxBackgroundConcurrency(InMaxBackgroundConcurrency)
	{
	}


	virtual int64 GetEstimatedMemory() const override
	{
		return TotalEstimatedMemory.load(std::memory_order_relaxed);
	}
	
	static int64 GetHardMemoryLimit()
	{
		return (int64)CVarAsyncAssetCompilationMaxMemoryUsage.GetValueOnAnyThread() * 1024ll * 1024ll * 1024ll;
	}
	static bool IsMemoryConstrained()
	{
		return GetHardMemoryLimit() != 0;
	}

	int64 GetDefaultMemoryPerAsset() const
	{
		return FMath::Max(0ll, (int64)CVarAsyncAssetCompilationMemoryPerCore.GetValueOnAnyThread(false) * 1024ll * 1024ll * 1024ll);
	}

	int64 GetMemoryAvailableForAssetCompilation() const
	{
		const FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
		
		// UsedPhysical = "working set"
		// UsedVirtual  = "commit charge"
		UE_LOGF(LogAsyncCompilation, Verbose, "GetMemoryAvailableForAssetCompilation UsedPhysical = %.3f MB (Peak %.3f MB) UsedVirtual = %.3f MB (Peak %.3f MB) AvailPhysical = %.3f MB AvailVirtual = %.3f MB ",
				MemoryStats.UsedPhysical/(1024*1024.f),
				MemoryStats.PeakUsedPhysical/(1024*1024.f),
				MemoryStats.UsedVirtual/(1024*1024.f),
				MemoryStats.PeakUsedVirtual/(1024*1024.f),
				MemoryStats.AvailablePhysical/(1024*1024.f),
				MemoryStats.AvailableVirtual/(1024*1024.f));

		// Just make sure available physical fits in a int64, if it's bigger than that, we're not expecting to be memory limited anyway
		// Also uses AvailableVirtual because the system might have plenty of physical memory but still be limited by virtual memory available in some cases.
		//   (i.e. per-process quota, paging file size lower than actual memory available, etc.).
		const int64 SystemAvailableMemory = (int64)FMath::Min3(MemoryStats.AvailablePhysical, MemoryStats.AvailableVirtual, (uint64)INT64_MAX);

		const int64 HardMemoryLimit = GetHardMemoryLimit();
		const int64 LeaveMemoryHeadRoom = 64 * 1024 * 1024; // leave 64 MB head room

		// No hard limit specified, use system memory
		if (HardMemoryLimit == 0) {
			return FMath::Max(0, SystemAvailableMemory - LeaveMemoryHeadRoom);
		}

		const int64 MemoryAvailableAgainstHardLimit = FMath::Max(0, HardMemoryLimit - TotalEstimatedMemory);

		if (SystemAvailableMemory < MemoryAvailableAgainstHardLimit) 
		{
			int64 CurrentDefecit = MemoryAvailableAgainstHardLimit - SystemAvailableMemory;

			if (CurrentDefecit > LargestDefecit) 
			{
				LargestDefecit = CurrentDefecit;
				UE_LOGF(LogAsyncCompilation, VeryVerbose, "Memory available for asset compilation less than exptected: (Hard Limit %.3f MB - SystemAvailableMemory %.3f MB) < MemoryAvailableAgainstHardLimit = %.3f MB",
					   HardMemoryLimit / (1024.f * 1024.f),
					   SystemAvailableMemory / (1024.f * 1024.f),
					   MemoryAvailableAgainstHardLimit / (1024.f * 1024.f));
			}
		}

		const int64 MemoryAvailableForAssetCompilation = FMath::Min(
			FMath::Max(0, MemoryAvailableAgainstHardLimit - LeaveMemoryHeadRoom), 
			FMath::Max(0, SystemAvailableMemory - LeaveMemoryHeadRoom)
		);

		return MemoryAvailableForAssetCompilation;
	}

	int64 GetRequiredMemory(const IQueuedWork* InWork) const
	{
		int64 RequiredMemory = InWork->GetRequiredMemory();

		// If the task doesn't support a memory estimate, use global defined default
		if (RequiredMemory == -1)
		{
			RequiredMemory = GetDefaultMemoryPerAsset(); // 4 GB by default
		}

		return RequiredMemory;
	}

	class MemorySharingInfo {
	private:
		const TFunction<int64 ()> GetAvailableMemory;
	public:
		int32 MemoryInMiBToBorrowFromSCW;	//	Memory in MiB to borrow from shader compilation when required for asset compilation.
		bool IsMemorySharingEnabled;

		MemorySharingInfo(TFunction<int64()> InGetAvailableMemory)
			: GetAvailableMemory(InGetAvailableMemory)
		{
			MemoryInMiBToBorrowFromSCW = GConfig->GetIntOrDefault(TEXT("AssetCompilation"), TEXT("MemoryToBorrowFromSCW"), 2000, GEngineIni);	
			IsMemorySharingEnabled = GConfig->GetBoolOrDefault(TEXT("AssetCompilation"), TEXT("SCWMemorySharingEnabled"), false, GEngineIni);

			const FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
			const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
			UE_LOGF(LogAsyncCompilation, Display, "MemorySharingInfo: UsedMemory=%lldMiB, MaxMemoryAvailable=%lldMiB", MemoryStats.UsedVirtual / 1024ull / 1024ull,	MemoryConstants.TotalVirtual / 1024ull / 1024ull);
		}	

		void LogMemoryInfo() const
		{
			const FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
			const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
			UE_LOGF(LogAsyncCompilation, Display, "MemorySharingInfo: UsedMemory=%lldMiB, MaxMemoryAvailable=%lldMiB Currently Available=%lldMiB", MemoryStats.UsedVirtual / 1024 / 1024,	MemoryConstants.TotalVirtual / 1024 / 1024, GetAvailableMemory() / 1024 / 1024);
		}

		bool IsSCWMemoryWithinLimit() const 
		{ 			
			FShaderCompileMemoryUsage SCWMemoryUsage = GShaderCompilingManager->GetExternalMemoryUsage();
			FShaderCompilerMemoryLimits SCWMemoryLimits = GetShaderCompilerCurrentMemoryLimits();		

			return SCWMemoryUsage.VirtualMemory <= static_cast<uint64>(SCWMemoryLimits.CurrentLimit * 1024ull * 1024ull);
		}
	};

	class FMemoryBoundScheduledWork : public FQueuedThreadPoolWrapper::FScheduledWork
	{
	public:
		FMemoryBoundScheduledWork(MemorySharingInfo& InMemoryInfo) 
		: MemoryInfo(InMemoryInfo)
		{
			FQueuedThreadPoolWrapper::FScheduledWork();
		}
		// Used to make sure this value remains constant between OnScheduled and OnUnscheduled
		int64 RequiredMemory = 0;
		uint64 TraceRegionId = 0;
		MemorySharingInfo& MemoryInfo;
	protected:
		void DoThreadedWork() override
		{						
			if (MemoryInfo.IsMemorySharingEnabled && FMemoryBoundQueuedThreadPoolWrapper::IsMemoryConstrained())
			{
				auto WaitForSCWMemory = [this]() 
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FMemoryBoundScheduledWork::WaitForSCWMemory);

					const float MaxWaitTimeSeconds = 2.0f;
					const float PerIterationWaitTimeSeconds  = 0.2f;
					const int WaitIterations = static_cast<int>(MaxWaitTimeSeconds / PerIterationWaitTimeSeconds );

					TRACE_COUNTER_INCREMENT(AsyncDelayedTask);
					TRACE_COUNTER_INCREMENT(AsyncDelayedTaskActive);			

					int Counter = 0;
					for (Counter = 0; Counter < WaitIterations; Counter++) 
					{
						if (MemoryInfo.IsSCWMemoryWithinLimit())
							break;

						FPlatformProcess::Sleep(PerIterationWaitTimeSeconds);
					}

					if (Counter > 0)
					{
						UE_LOGF(LogAsyncCompilation, Display, "Waited for SCW memory: %d iterations for a total of %f seconds",
							   Counter,
							   Counter * PerIterationWaitTimeSeconds);


						if (Counter == WaitIterations)
						{
							UE_LOGF(LogAsyncCompilation, Display, "Waited for the maximum allowed time for SCW to reduce memory usage, proceeding with work anyway");
						}
					}
				
					TRACE_COUNTER_DECREMENT(AsyncDelayedTaskActive);
				};

				if (RequiredMemory > FMemoryBoundQueuedThreadPoolWrapper::GetHardMemoryLimit())
				{
					SetShaderCompilerAssetMemoryLoanInMiB(MemoryInfo.MemoryInMiBToBorrowFromSCW);
					WaitForSCWMemory();
					FQueuedThreadPoolWrapper::FScheduledWork::DoThreadedWork();
					SetShaderCompilerAssetMemoryLoanInMiB(0);
				}
				else
				{
					FQueuedThreadPoolWrapper::FScheduledWork::DoThreadedWork();
				}
			} 
			else 
			{
				FQueuedThreadPoolWrapper::FScheduledWork::DoThreadedWork();
			}
		}
	};

	FScheduledWork* AllocateScheduledWork() override
	{
		return new FMemoryBoundScheduledWork(MemoryInfo);
	}

	void UpdateCounters()
	{
		TRACE_COUNTER_SET(AsyncCompilationAvailableMemory, GetMemoryAvailableForAssetCompilation());
		TRACE_COUNTER_SET(AsyncCompilationTotalEstimatedMemory, TotalEstimatedMemory);
		TRACE_COUNTER_SET(AsyncCompilationConcurrency, GetCurrentConcurrency());
	}

	void OnScheduled(const IQueuedWork* InWork) override
	{
		FMemoryBoundScheduledWork* Work = (FMemoryBoundScheduledWork*)InWork;
		Work->RequiredMemory = GetRequiredMemory(InWork);
		TotalEstimatedMemory += Work->RequiredMemory; // atomic
		
		const TCHAR * DebugName = Work->GetDebugName();
		if ( DebugName == nullptr )
		{
			DebugName = TEXT("No DebugName");
		}

		Work->TraceRegionId = TRACE_BEGIN_REGION_WITH_ID(*FString::Printf(TEXT("%s (%.3f MiB)"), DebugName, Work->RequiredMemory/(1024*1024.f)), TEXT("AsyncCompilation"));

		UE_LOGF(LogAsyncCompilation, Verbose, "OnScheduled [%ls] Work->RequiredMemory = %.3f MB TotalEstimatedMemory = %.3f MB (TraceId %lld)",
				DebugName,
				Work->RequiredMemory/(1024.f*1024.f),
				TotalEstimatedMemory/(1024.f*1024.f),
				Work->TraceRegionId
			);

		UpdateCounters();
	}

	void OnUnscheduled(const IQueuedWork* InWork) override
	{
		FMemoryBoundScheduledWork* Work = (FMemoryBoundScheduledWork*)InWork;
		TotalEstimatedMemory -= Work->RequiredMemory; // atomic
		check(TotalEstimatedMemory >= 0);

		if (Work->TraceRegionId)
		{
			TRACE_END_REGION_WITH_ID(Work->TraceRegionId);
		}
		
		// don't call Work->GetDebugName , it's gone now
		UE_LOGF(LogAsyncCompilation, Verbose, "OnUnscheduled Work->RequiredMemory = %.3f MB TotalEstimatedMemory = %.3f MB (TraceId %lld)",
				Work->RequiredMemory/(1024.f*1024.f),
				TotalEstimatedMemory/(1024.f*1024.f),
				Work->TraceRegionId
			);

		UpdateCounters();
	}

	int32 GetAdjustedMaxConcurrency(EQueuedWorkPriority Priority) const
	{
		// Only unlocks foreground concurrency when priority is highest or blocking. This improves
		// editor responsiveness under heavy load when GT is blocking on something and
		// max concurrency has already been reached by background work.
		if (Priority <= EQueuedWorkPriority::Highest)
		{
			return MaxBackgroundConcurrency() + MaxForegroundConcurrency();
		}
		else
		{
			return MaxBackgroundConcurrency();
		}
	}

	int32 GetMaxConcurrency() const override
	{
		int64 NewRequiredMemory = 0;
		
		// Add next work memory requirement to see if it still fits in available memory
		EQueuedWorkPriority Priority;
		IQueuedWork* NextWork = QueuedWork.Peek(&Priority);
		if (NextWork)
		{
			NewRequiredMemory += GetRequiredMemory(NextWork);
		}

		int32 DynamicMaxConcurrency = GetAdjustedMaxConcurrency(Priority);
		int32 Concurrency = FQueuedThreadPoolWrapper::GetCurrentConcurrency();
		
		int64 AvailableAssetCompilationMemory = GetMemoryAvailableForAssetCompilation();

		// we check MemoryLimit which uses current system available memory against TotalRequiredMemory
		//	which includes already running tasks
		// those tasks may have already allocated, so they have been counted twice
		//  this means we can count tasks as using twice as much memory as they actually do
		//  eg. one 4 GB task can block all concurrency when there was 8 GB of room
		//    because after it allocated there is 4 GB free but we also then subtract off the 4 GB the task requires
		// this is conservative, so okay if we want to avoid OOM and are okay with less concurrency than possible

		if ( NewRequiredMemory > 0 && NewRequiredMemory >= AvailableAssetCompilationMemory )
		{
			if ( Concurrency == 0 )
			{
				// Never limit below a concurrency of 1 or else we'll starve the asset processing and never be scheduled again
				// The idea for now is to let assets get built one by one when starving on memory, which should not increase OOM compared to the old synchronous behavior.
				const TCHAR* DebugName = nullptr;
				if (NextWork)
				{
					DebugName = NextWork->GetDebugName();
				}
				if (DebugName == nullptr)
				{
					DebugName = TEXT("No DebugName");
				}

				const bool bHasMemoryEstimate = NextWork && NextWork->GetRequiredMemory() >= 0;
				if (bHasMemoryEstimate)
				{
					const int64 HardMemoryLimit = GetHardMemoryLimit();
					if (NewRequiredMemory > HardMemoryLimit && HardMemoryLimit > 0)
					{
						// Separately report the case if an asset was bigger than our manually set memory limit. Such assets are always too big,
						// and don't just exceed the currently available memory.
						UE_LOGFMT_NSLOC(LogAsyncCompilation, Warning, "AsyncAssetCompilation", "HardMemoryLimitExceeded",
							"BEWARE: AssetCompile memory estimate is greater than the hard memory limit, but we're running it [{TaskName}] anyway! "
							"RequiredMemory = {TotalEstimatedMemory} MiB + {RequiredMemory} MiB, AvailableAssetCompilationMemory = {AvailableAssetCompilationMemory} MiB, HardMemoryLimit = {HardMemoryLimit} MiB",
							("TaskName", DebugName),
							("RequiredMemory", FString::SanitizeFloat(NewRequiredMemory / (1024.f * 1024.f), 3)),
							("TotalEstimatedMemory", FString::SanitizeFloat(TotalEstimatedMemory / (1024.f * 1024.f), 3)),
							("AvailableAssetCompilationMemory", FString::SanitizeFloat(AvailableAssetCompilationMemory / (1024.f * 1024.f), 3)),
							("HardMemoryLimit", FString::SanitizeFloat(HardMemoryLimit / (1024.f * 1024.f), 3))
						);
					}
					else
					{
						UE_LOGFMT_NSLOC(LogAsyncCompilation, Display, "AsyncAssetCompilation", "MemoryLimitExceeded",
							"BEWARE: AssetCompile memory estimate is greater than available, but we're running it [{TaskName}] anyway! "
							"RequiredMemory = {TotalEstimatedMemory} MiB + {RequiredMemory} MiB, MemoryLimit = {AvailableAssetCompilationMemory} MiB ",
							("TaskName", DebugName),
							("RequiredMemory", FString::SanitizeFloat(NewRequiredMemory / (1024.f * 1024.f), 3)),
							("TotalEstimatedMemory", FString::SanitizeFloat(TotalEstimatedMemory / (1024.f * 1024.f), 3)),
							("AvailableAssetCompilationMemory", FString::SanitizeFloat(AvailableAssetCompilationMemory / (1024.f * 1024.f), 3))
						);
					}
				}

				// @todo : ? pause the main thread? pause shader compilers? trigger a GC ?

				DynamicMaxConcurrency = 1;
				// this will cause CanSchedule() to return true
			}
			else
			{
				UE_LOGF(LogAsyncCompilation, Verbose, "CONCURRENCY LIMITED to %d/%d; RequiredMemory = %.3f MB + %.3f MB AvailableAssetCompilationMemory = %.3f MB ",
					Concurrency,DynamicMaxConcurrency,
					NewRequiredMemory/(1024*1024.f),
					TotalEstimatedMemory/(1024*1024.f),
					AvailableAssetCompilationMemory/(1024*1024.f));
					
				DynamicMaxConcurrency = Concurrency; // Limit concurrency to what we already allowed
				// this will cause CanSchedule() to return false
			}
		}

		TRACE_COUNTER_SET(AsyncCompilationMaxConcurrency, DynamicMaxConcurrency);

		return DynamicMaxConcurrency;
	}
	
	void AddQueuedWork(IQueuedWork* InQueuedWork, EQueuedWorkPriority InPriority) override
	{
		// this is when the work is added to our queue

		int64 RequiredMemory = InQueuedWork->GetRequiredMemory();
		const TCHAR * DebugName = InQueuedWork->GetDebugName();
		
		if ( DebugName == nullptr )
		{
			DebugName = TEXT("No DebugName");
		}

		if ( RequiredMemory <= 0 )
		{
			// log the types that are failing to set required memory
			UE_LOGF(LogAsyncCompilation, Display, "AddQueuedWork [%ls] did not supply a RequiredMemory estimate. This is a bug, all asset types should provide an estimate. When no estimate is provided asset compilation is slowed as the queue has to operate very conservatively.",DebugName);
		}
		else
		{
			UE_LOGF(LogAsyncCompilation, Verbose, "AddQueuedWork [%ls] RequiredMemory = %lld",DebugName,RequiredMemory);
		}

		FQueuedThreadPoolWrapper::AddQueuedWork(InQueuedWork,InPriority);
	}

private:
	MemorySharingInfo MemoryInfo;

	TFunction<int32()> MaxForegroundConcurrency;
	TFunction<int32()> MaxBackgroundConcurrency;

	std::atomic<int64> TotalEstimatedMemory {0};
	mutable std::atomic<int64> LargestDefecit {0};
};

#endif // #if WITH_EDITOR

TArrayView<IAssetCompilingManager* const> FAssetCompilingManager::GetRegisteredManagers() const
{
	return TArrayView<IAssetCompilingManager* const>(AssetCompilingManagers);
}

FAssetCompilingManager::FAssetCompilingManager()
{
#if WITH_EDITOR
	AssetCompilingManagerImpl::EnsureInitializedCVars();

	RegisterManager(&FStaticMeshCompilingManager::Get());
	RegisterManager(&FSkinnedAssetCompilingManager::Get());
	RegisterManager(&FTextureCompilingManager::Get());
	RegisterManager(&FActorDeferredScriptManager::Get());
	RegisterManager(&FSoundWaveCompilingManager::Get());
	RegisterManager(&UE::Anim::FAnimSequenceCompilingManager::Get());
	RegisterManager(&FAnimBankCompilingManager::Get());
	RegisterManager(&FAnimSequenceTransformProviderCompilingManager::Get());

	// Ensure that the thread pool is constructed before the memory queue is registered because
	// the memory queue can call GetThreadPool() from a worker thread, and could lead to a race
	// to create the thread pool.
	GetThreadPool();

	IModularFeatures::Get().RegisterModularFeature(UE::DerivedData::IBuildSchedulerMemoryQueue::FeatureName, &AssetCompilingManagerImpl::GMemoryQueue);
#endif
}

FAssetCompilingManager::~FAssetCompilingManager()
{
#if WITH_EDITOR
	IModularFeatures::Get().UnregisterModularFeature(UE::DerivedData::IBuildSchedulerMemoryQueue::FeatureName, &AssetCompilingManagerImpl::GMemoryQueue);
#endif
}

extern CORE_API int32 GNumForegroundWorkers; // TaskGraph.cpp

FQueuedThreadPool* FAssetCompilingManager::GetThreadPool() const
{
#if WITH_EDITOR
	static FQueuedThreadPoolWrapper* GAssetThreadPool = nullptr;
	if (GAssetThreadPool == nullptr && GLargeThreadPool != nullptr)
	{
		// We limit concurrency to half the workers because asset compilation is hard on total memory and memory bandwidth and can run slower when going wider than actual cores.
		// Most asset also have some form of inner parallelism during compilation.
		// Recently found out that GThreadPool and GLargeThreadPool have the same amount of workers, so can't rely on GThreadPool to be our limiter here.
		// FPlatformMisc::NumberOfCores() and FPlatformMisc::NumberOfCoresIncludingHyperthreads() also return the same value when -corelimit is used so we can't use FPlatformMisc::NumberOfCores()
		// if we want to keep the same 1:2 relationship with worker count.
		auto MaxBackgroundConcurrency = []() { return FMath::Max(1, GLargeThreadPool->GetNumThreads() / 2); };

		// Additional concurrency to unlock at highest priority
		auto MaxForegroundConcurrency = []() { return FMath::Max(0, GNumForegroundWorkers); };

		// All asset priorities will resolve to a Low priority once being scheduled.
		// Any asset supporting being built async should be scheduled lower than Normal to let non-async stuff go first
		// However, we let Highest and Blocking priority pass-through as it to benefit from going to foreground threads when required (i.e. Game-thread is waiting on some assets)
		GAssetThreadPool = new FMemoryBoundQueuedThreadPoolWrapper(GLargeThreadPool, MaxForegroundConcurrency, MaxBackgroundConcurrency, [](EQueuedWorkPriority Priority) { return Priority <= EQueuedWorkPriority::Highest ? Priority : EQueuedWorkPriority::Low; });

		AsyncCompilationHelpers::BindThreadPoolToCVar(
			GAssetThreadPool,
			CVarAsyncCompilationStandard.AsyncCompilation,
			CVarAsyncCompilationStandard.AsyncCompilationResume,
			CVarAsyncCompilationStandard.AsyncCompilationMaxConcurrency
		);
	}

	return GAssetThreadPool;
#else
	return GThreadPool;
#endif
}

/**
 * Register an asset compiling manager.
 */
bool FAssetCompilingManager::RegisterManager(IAssetCompilingManager* InAssetCompilingManager)
{
	if (AssetCompilingManagers.Contains(InAssetCompilingManager))
	{
		return false;
	}

	AssetCompilingManagers.Add(InAssetCompilingManager);
	AssetCompilingManagersWithValidDependencies.Add(InAssetCompilingManager);

	auto GetVertexDependencies =
		[this](IAssetCompilingManager* CurrentManager)
	{
		TArray<IAssetCompilingManager*> Connections;
		for (FName DependentTypeName : CurrentManager->GetDependentTypeNames())
		{
			IAssetCompilingManager** DependentManager =
				Algo::FindBy(AssetCompilingManagersWithValidDependencies, DependentTypeName, [](const IAssetCompilingManager* Manager) { return Manager->GetAssetTypeName();	});

			if (DependentManager)
			{
				Connections.Add(*DependentManager);
			}
		}
		
		return Connections;
	};

	if (!Algo::TopologicalSort(AssetCompilingManagers, GetVertexDependencies))
	{
		AssetCompilingManagersWithValidDependencies.Remove(InAssetCompilingManager);
		UE_LOGF(LogAsyncCompilation, Error, "AssetCompilingManager %ls introduced a circular dependency, it's dependencies will be ignored", *InAssetCompilingManager->GetAssetTypeName().ToString());
	}

	return true;
}

bool FAssetCompilingManager::UnregisterManager(IAssetCompilingManager* InAssetCompilingManager)
{
	if (!AssetCompilingManagers.Contains(InAssetCompilingManager))
	{
		return false;
	}

	AssetCompilingManagers.Remove(InAssetCompilingManager);
	AssetCompilingManagersWithValidDependencies.Remove(InAssetCompilingManager);
	return true;
}

/**
 * Returns the number of outstanding asset compilations.
 */
int32 FAssetCompilingManager::GetNumRemainingAssets() const
{
	int32 RemainingAssets = 0;
	for (IAssetCompilingManager* AssetCompilingManager : AssetCompilingManagers)
	{
		RemainingAssets += AssetCompilingManager->GetNumRemainingAssets();
	}

	return RemainingAssets;
}

/**
 * Blocks until completion of all assets.
 */
void FAssetCompilingManager::FinishAllCompilation()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAssetCompilingManager::FinishAllCompilation");
	for (IAssetCompilingManager* AssetCompilingManager : AssetCompilingManagers)
	{
		AssetCompilingManager->FinishAllCompilation();
	}

	UpdateNumRemainingAssets();
}

/**
 * Finish compilation of the requested objects.
 */
void FAssetCompilingManager::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
#if WITH_EDITOR
	if (InObjects.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAssetCompilingManager::FinishCompilationForObjects);

		for (IAssetCompilingManager* AssetCompilingManager : AssetCompilingManagers)
		{
			AssetCompilingManager->FinishCompilationForObjects(InObjects);
		}

		bool bAreObjectsStillCompiling = false;
		for (const UObject* Object : InObjects)
		{
			if (const IInterface_AsyncCompilation* AsyncCompilationInterface = Cast<IInterface_AsyncCompilation>(Object))
			{
				if (AsyncCompilationInterface->IsCompiling())
				{
					bAreObjectsStillCompiling = true;

					ensureMsgf(false,
						TEXT("A finish compilation has been requested on %s but it is still being compiled, this might indicate a missing implementation of IAssetCompilingManager::FinishCompilationForObjects"),
						*Object->GetFullName());

					break;
				}
			}
		}

		if (bAreObjectsStillCompiling)
		{
			FinishAllCompilation();
		}
	}
#endif
}

/**
 * Finish compilation of the requested objects.
 */
void FAssetCompilingManager::MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects)
{
#if WITH_EDITOR
	if (InObjects.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAssetCompilingManager::MarkCompilationAsCanceled);

		for (IAssetCompilingManager* AssetCompilingManager : AssetCompilingManagers)
		{
			AssetCompilingManager->MarkCompilationAsCanceled(InObjects);
		}
	}
#endif
}

/**
 * Cancel any pending work and blocks until it is safe to shut down.
 */
void FAssetCompilingManager::Shutdown()
{
	for (IAssetCompilingManager* AssetCompilingManager : AssetCompilingManagers)
	{
		AssetCompilingManager->Shutdown();
	}

#if WITH_EDITOR
	if (FParse::Param(FCommandLine::Get(), TEXT("DumpAsyncStallsOnExit")))
	{
		AsyncCompilationHelpers::DumpStallStacks();
	}
#endif // #if WITH_EDITOR
}

FAssetCompilingManager& FAssetCompilingManager::Get()
{
	static FAssetCompilingManager Singleton;
	return Singleton;
}

void FAssetCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	// Reuse ObjectIterator Caching and Reverse lookups for the duration of all asset updates
	FObjectCacheContextScope ObjectCacheScope;

	for (IAssetCompilingManager* AssetCompilingManager : AssetCompilingManagers)
	{
		AssetCompilingManager->ProcessAsyncTasks(bLimitExecutionTime);
	}

	UpdateNumRemainingAssets();
}

void FAssetCompilingManager::ProcessAsyncTasks(const AssetCompilation::FProcessAsyncTaskParams& Params)
{
	// Reuse ObjectIterator Caching and Reverse lookups for the duration of all asset updates
	FObjectCacheContextScope ObjectCacheScope;

	for (IAssetCompilingManager* AssetCompilingManager : AssetCompilingManagers)
	{
		AssetCompilingManager->ProcessAsyncTasks(Params);
	}

	UpdateNumRemainingAssets();
}

void FAssetCompilingManager::UpdateNumRemainingAssets()
{
	const int32 NumRemainingAssets = GetNumRemainingAssets();
	if (LastNumRemainingAssets > 0 && NumRemainingAssets == 0)
	{
		// This is important to at least broadcast once we reach 0 remaining assets
		// because some listener are only interested to be notified when the number
		// of total async compilation reaches 0.
		FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast({});
	}
	LastNumRemainingAssets = NumRemainingAssets;

#if WITH_EDITOR
	// Hold a Slate responsive-mode request while there is pending work, so the editor ticks at full rate
	// and the per-frame processing budget is consumed as fast as possible. Without this, Slate can
	// throttle itself during load (no user input), causing asset compilation to take far longer than necessary.
	if (NumRemainingAssets > 0)
	{
		if (!ResponsiveModeHandle.IsValid() && FSlateApplication::IsInitialized())
		{
			ResponsiveModeHandle = FSlateThrottleManager::Get().EnterResponsiveMode();
		}
	}
	else if (ResponsiveModeHandle.IsValid())
	{
		FSlateThrottleManager::Get().LeaveResponsiveMode(ResponsiveModeHandle);
	}
#endif
}

#undef LOCTEXT_NAMESPACE
