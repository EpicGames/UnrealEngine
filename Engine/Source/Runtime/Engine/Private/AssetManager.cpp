// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/AssetManager.h"

#include "Algo/Compare.h"
#include "Algo/Transform.h"
#include "Algo/Unique.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetBundleData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/AssetManagerSettings.h"
#include "Engine/AssetManagerTypes.h"
#include "Engine/BlueprintCore.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/GameInstance.h"
#include "Engine/StreamableManager.h"
#include "Engine/Experimental/StreamableManagerError.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "IPlatformFilePak.h"
#include "Logging/StructuredLog.h"
#include "Misc/CoreDelegates.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FileHelper.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "Misc/Launder.h"
#include "MoviePlayerProxy.h"
#include "Modules/ModuleManager.h"
#include "Stats/StatsMisc.h"
#include "String/Find.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/ICookInfo.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetManager)

#if WITH_EDITOR
#include "Editor.h"
#include "Misc/ChunkDependencyInfo.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Widgets/Notifications/SNotificationList.h"
#else
#include "Engine/Engine.h"
#include "Serialization/MemoryReader.h"
#endif

#define LOCTEXT_NAMESPACE "AssetManager"
LLM_DEFINE_TAG(AssetManager);

DEFINE_LOG_CATEGORY(LogAssetManager);

#define UE_ASSETMANAGER_SMALL_LOADSTATE 1

#if UE_ASSETMANAGER_SMALL_LOADSTATE

/** Structure defining the current loading state of an asset */
struct FPrimaryAssetLoadState
{
private:
	// Note: this implementation reduces memory by moving as much of the state as possible into a single buffer.
	// If the state is empty, this buffer is not allocated so we don't pay the memory for it.
	// Layout is [ (TSharedPtr<FStreamableHandle>) | (NumBundleNames * sizeof(FName)) | (optional FAssetManagerDownloadParams)]

	/** The storage buffer */
	uint8* Storage = nullptr;
	/** Number of bundles in the storage buffer */
	uint16 NumBundleNames = 0;
	/** If true, the buffer contains download params */
	bool bWithDownloadParams = 0;

	constexpr static SIZE_T GetBundleNamesOffset()
	{ 
		return Align(sizeof(TSharedPtr<FStreamableHandle>), alignof(FName));
	}

	static SIZE_T GetDownloadParamsOffset(uint16 InNumBundles)
	{
		return Align(GetBundleNamesOffset() + sizeof(FName) * InNumBundles, alignof(FAssetManagerDownloadParams));
	}

	void ReallocStorage(uint16 InNumBundles, bool bInWithDownloadParams)
	{
		FMemory::Free(Storage);

		const SIZE_T StorageSize = bInWithDownloadParams
			? (GetDownloadParamsOffset(InNumBundles) + sizeof(FAssetManagerDownloadParams))
			: (GetBundleNamesOffset() + sizeof(FName) * InNumBundles);

		Storage = reinterpret_cast<uint8*>(FMemory::Malloc(StorageSize));
	}

	bool NeedsRealloc(uint16 InNumBundles, bool bInWithDownloadParams) const
	{
		return !Storage || InNumBundles != NumBundleNames || bInWithDownloadParams != bWithDownloadParams;
	}

	void DestroyStoredItems()
	{
		check(Storage);
		DestructItem(GetHandleInternal());
		DestructItems(GetBundleNamesInternal(), NumBundleNames);
		if (FAssetManagerDownloadParams* DownloadParams = GetDownloadParamsInternal())
		{
			DestructItem(DownloadParams);
		}
	}

	/** The handle to the streamable state for this asset, this keeps the objects in memory. If handle is invalid, not in memory at all */
	TSharedPtr<FStreamableHandle>* GetHandleInternal()
	{
		check(Storage);
		return UE_LAUNDER(reinterpret_cast<TSharedPtr<FStreamableHandle>*>(Storage));
	}

	const TSharedPtr<FStreamableHandle>* GetHandleInternal() const
	{
		return const_cast<FPrimaryAssetLoadState*>(this)->GetHandleInternal();
	}

	/** The set of bundles to be loaded by the handle */
	FName* GetBundleNamesInternal()
	{
		check(Storage);
		return UE_LAUNDER(reinterpret_cast<FName*>(Storage + GetBundleNamesOffset()));
	}

	const FName* GetBundleNamesInternal() const
	{
		return const_cast<FPrimaryAssetLoadState*>(this)->GetBundleNamesInternal();
	}

	/** Params for installing assets before loading */
	FAssetManagerDownloadParams* GetDownloadParamsInternal()
	{
		if (bWithDownloadParams)
		{
			check(Storage);
			return UE_LAUNDER(reinterpret_cast<FAssetManagerDownloadParams*>(Storage + GetDownloadParamsOffset(NumBundleNames)));
		}

		return nullptr;
	}

	const FAssetManagerDownloadParams* GetDownloadParamsInternal() const
	{
		return const_cast<FPrimaryAssetLoadState*>(this)->GetDownloadParamsInternal();
	}

	inline static const TSharedPtr<FStreamableHandle> InvalidHandle = {};

public:
	FPrimaryAssetLoadState() = default;
	FPrimaryAssetLoadState(const FPrimaryAssetLoadState&) = delete;

	FPrimaryAssetLoadState(FPrimaryAssetLoadState&& Other) 
		: Storage(Other.Storage)
		, NumBundleNames(Other.NumBundleNames)
		, bWithDownloadParams(Other.bWithDownloadParams)
	{
		Other.Storage = nullptr;
		Other.NumBundleNames = 0;
		Other.bWithDownloadParams = false;
	}

	FPrimaryAssetLoadState& operator=(const FPrimaryAssetLoadState&) = delete;

	FPrimaryAssetLoadState& operator=(FPrimaryAssetLoadState&& Other)
	{
		if (Storage)
		{
			DestroyStoredItems();
			FMemory::Free(Storage);
		}

		Storage = Other.Storage;
		NumBundleNames = Other.NumBundleNames;
		bWithDownloadParams = Other.bWithDownloadParams;

		Other.Storage = nullptr;
		Other.NumBundleNames = 0;
		Other.bWithDownloadParams = false;

		return *this;
	}

	~FPrimaryAssetLoadState()
	{
		if (Storage)
		{
			DestroyStoredItems();
			FMemory::Free(Storage);
			Storage = nullptr;
		}
	}

	bool IsSame(TConstArrayView<FName> InBundleNames, const TOptional<FAssetManagerDownloadParams>& InDownloadParams) const
	{
		if (NumBundleNames == InBundleNames.Num() && bWithDownloadParams == InDownloadParams.IsSet())
		{
			// bWithDownloadParams can only be true if storage is allocated
			if (bWithDownloadParams && *GetDownloadParamsInternal() != InDownloadParams)
			{
				return false;
			}

			return Algo::Compare(InBundleNames, GetBundleNames());
		}

		return false;
	}

	void SetState(TSharedPtr<FStreamableHandle> InHandle, TConstArrayView<FName> InBundleNames, TOptional<FAssetManagerDownloadParams> InDownloadParams)
	{
		const uint16 InNumBundleNames = IntCastChecked<uint16>(InBundleNames.Num());

		if (NeedsRealloc(InNumBundleNames, InDownloadParams.IsSet()))
		{
			if (Storage)
			{
				DestroyStoredItems();
			}
			ReallocStorage(InNumBundleNames, InDownloadParams.IsSet());
			NumBundleNames = InNumBundleNames;
			bWithDownloadParams = InDownloadParams.IsSet();

			new(Storage) TSharedPtr<FStreamableHandle>(MoveTemp(InHandle));

			ConstructItems<FName>(GetBundleNamesInternal(), InBundleNames.GetData(), InNumBundleNames);

			if (InDownloadParams)
			{
				new(GetDownloadParamsInternal()) FAssetManagerDownloadParams(MoveTemp(*InDownloadParams));
			}
		}
		else
		{
			*GetHandleInternal() = MoveTemp(InHandle);

			CopyAssignItems(GetBundleNamesInternal(), InBundleNames.GetData(), InNumBundleNames);

			if (InDownloadParams)
			{
				*GetDownloadParamsInternal() = MoveTemp(*InDownloadParams);
			}
		}
	}

	void MoveStateFrom(TSharedPtr<FStreamableHandle> InHandle, FPrimaryAssetLoadState&& InOther)
	{
		*this = MoveTemp(InOther);
		if (Storage)
		{
			*GetHandleInternal() = MoveTemp(InHandle);
		}
	}

	const TSharedPtr<FStreamableHandle>& GetHandle() const UE_LIFETIMEBOUND
	{
		if (Storage)
		{
			return *GetHandleInternal();
		}
		return InvalidHandle;
	}

	TConstArrayView<FName> GetBundleNames() const UE_LIFETIMEBOUND
	{ 
		if (Storage)
		{
			return MakeArrayView(GetBundleNamesInternal(), NumBundleNames);
		}
		return {};
	}

	/** If this state is keeping things in memory */
	bool IsValid() const 
	{
		const TSharedPtr<FStreamableHandle>& Handle = GetHandle();
		return Handle.IsValid() && Handle->IsActive();
	}

	/** Reset this state */
	void Reset(bool bCancelHandle)
	{
		const TSharedPtr<FStreamableHandle>& Handle = GetHandle();
		if (Handle.IsValid())
		{
			if (Handle->IsActive() && bCancelHandle)
			{
				// This will call the cancel callback if set
				Handle->CancelHandle();
			}
		}
		
		if (Storage)
		{
			DestroyStoredItems();
			FMemory::Free(Storage);
			Storage = nullptr;
		}

		NumBundleNames = 0;
		bWithDownloadParams = false;
	}
};

#else

/** Structure defining the current loading state of an asset */
struct FPrimaryAssetLoadState
{
private:
	/** The handle to the streamable state for this asset, this keeps the objects in memory. If handle is invalid, not in memory at all */
	TSharedPtr<FStreamableHandle> Handle;

	/** The set of bundles to be loaded by the handle */
	TArray<FName> BundleNames;

	/** Params for installing assets before loading */
	TOptional<FAssetManagerDownloadParams> DownloadParams;

public:
	bool IsSame(TConstArrayView<FName> InBundleNames, const TOptional<FAssetManagerDownloadParams>& InDownloadParams) const
	{
		return Algo::Compare(InBundleNames, BundleNames) && (InDownloadParams == DownloadParams);
	}

	void SetState(TSharedPtr<FStreamableHandle> InHandle, TConstArrayView<FName> InBundleNames, TOptional<FAssetManagerDownloadParams> InDownloadParams)
	{
		Handle = MoveTemp(InHandle);
		BundleNames = InBundleNames;
		DownloadParams = MoveTemp(InDownloadParams);
	}

	void MoveStateFrom(TSharedPtr<FStreamableHandle> InHandle, FPrimaryAssetLoadState&& InOther)
	{
		Handle = MoveTemp(InHandle);
		BundleNames = MoveTemp(InOther.BundleNames);
		DownloadParams = MoveTemp(InOther.DownloadParams);
	}

	const TSharedPtr<FStreamableHandle>& GetHandle() const { return Handle; }

	const TArray<FName>& GetBundleNames() const { return BundleNames; }


	/** If this state is keeping things in memory */
	bool IsValid() const { return Handle.IsValid() && Handle->IsActive(); }

	/** Reset this state */
	void Reset(bool bCancelHandle)
	{
		if (Handle.IsValid())
		{
			if (Handle->IsActive() && bCancelHandle)
			{
				// This will call the cancel callback if set
				Handle->CancelHandle();
			}
			
			Handle = nullptr;
		}
		BundleNames.Reset();
		DownloadParams.Reset();
	}
};

#endif // UE_ASSETMANAGER_SMALL_LOADSTATE

struct FPrimaryAssetTypeData;

/** Structure representing data about a specific asset */
struct FPrimaryAssetData
{
public:
	FPrimaryAssetData() {}

	/** Path to this asset on disk */
	const FSoftObjectPtr& GetAssetPtr() const { return AssetPtr; }
	/** Path used to look up cached asset data in the asset registry. This will be missing the _C for blueprint classes */
	const FSoftObjectPath& GetARLookupPath() const { return ARLookupPath; }
	/** Asset is considered loaded at all if there is an active handle for it */
	bool IsLoaded() const { return CurrentState.IsValid(); }

private:
	// These are used as the keys in a reverse map and cannot be modified except when modifying the reversemap as well
	FSoftObjectPtr AssetPtr;
	FSoftObjectPath ARLookupPath;

public:
	/** Current state of this asset */
	FPrimaryAssetLoadState CurrentState;
	/** Pending state of this asset, will be copied to CurrentState when load finishes */
	FPrimaryAssetLoadState PendingState;

	friend FPrimaryAssetTypeData;
};

/** Structure representing all items of a specific asset type */
struct FPrimaryAssetTypeData
{
	/** The public info struct */
	FPrimaryAssetTypeInfo Info;

	// Each PrimaryAssetType has a list of PrimaryAssetNames that are PrimaryAssets of its type, and a map to store a
	// FPrimaryAssetData for each one, including the SoftObjectPtr to the asset represented by that Name.
	// The AssetManager also has a reverse map from SoftObjectPath to the PrimaryAssetId
	// (PrimaryAssetTypeName + PrimaryAssetName) for every PrimaryAsset.
	// We need to modify the list of assets and the reverse map together, so direct access to this->AssetMap is
	// private, and modifying it is done through functions that take the ReverseMap as an additional In/Out argument.
	const TMap<FName, FPrimaryAssetData>& GetAssets() const;
	FPrimaryAssetData& FindOrAddAsset(FName AssetName, const FSoftObjectPtr& AssetPtr, const FSoftObjectPath& ARLookupPath,
		TMap<FSoftObjectPath, FPrimaryAssetId>& InOutReverseMap);
	void RemoveAsset(FName AssetName, TMap<FSoftObjectPath, FPrimaryAssetId>& InOutReverseMap);
	void RemoveAssets(TConstArrayView<FName> AssetNames, TMap<FSoftObjectPath, FPrimaryAssetId>& InOutReverseMap);
	void ResetAssets(TMap<FSoftObjectPath, FPrimaryAssetId>& InOutReverseMap);
	void ShrinkAssets();

	/** In the editor, paths that we need to scan once asset registry is done loading */
	TSet<FString> DeferredAssetScanPaths;

	/** List of paths that were explicitly requested by other systems and not loaded from the default config */
	TSet<FString> AdditionalAssetScanPaths;

	/** Expanded list of asset scan paths and package names, will not include virtual paths */
	TSet<FString> RealAssetScanPaths;

	FPrimaryAssetTypeData() {}
	~FPrimaryAssetTypeData();

	FPrimaryAssetTypeData(FName InPrimaryAssetType, UClass* InAssetBaseClass, bool bInHasBlueprintClasses, bool bInIsEditorOnly)
		: Info(InPrimaryAssetType, InAssetBaseClass, bInHasBlueprintClasses, bInIsEditorOnly)
		{}

private:
	/** Map of scanned assets */
	TMap<FName, FPrimaryAssetData> AssetMap;
};

FPrimaryAssetTypeData::~FPrimaryAssetTypeData()
{
	// Assets must be removed via a call to e.g. ResetAssets before destruction, so that the AssetManager's AssetPathMap
	// can be updated. For an example, see UAssetManager::RemovePrimaryAssetType
	checkf(AssetMap.IsEmpty(), TEXT("FPrimaryAssetTypeData is being destructed while still containing assets"));
}

const TMap<FName, FPrimaryAssetData>& FPrimaryAssetTypeData::GetAssets() const
{
	return AssetMap;
}

FPrimaryAssetData& FPrimaryAssetTypeData::FindOrAddAsset(FName AssetName, const FSoftObjectPtr& AssetPtr,
	const FSoftObjectPath& ARLookupPath, TMap<FSoftObjectPath, FPrimaryAssetId>& InOutReverseMap)
{
	FPrimaryAssetData& ExistingAssetData = AssetMap.FindOrAdd(AssetName);
	const FSoftObjectPath& OldAssetRef = ExistingAssetData.AssetPtr.ToSoftObjectPath();
	// If the old reference exists and we are replacing it, remove it from the reverse map before adding the new one
	if (!OldAssetRef.IsNull())
	{
		InOutReverseMap.Remove(OldAssetRef);
	}

	ExistingAssetData.AssetPtr = AssetPtr;
	ExistingAssetData.ARLookupPath = ARLookupPath;

	const FSoftObjectPath& NewAssetRef = ExistingAssetData.AssetPtr.ToSoftObjectPath();
	// Dynamic types can have null asset refs NewAssetRef might be null
	if (!NewAssetRef.IsNull())
	{
		InOutReverseMap.Add(NewAssetRef, FPrimaryAssetId(Info.PrimaryAssetType, AssetName));
	}
	Info.NumberOfAssets = AssetMap.Num();

	return ExistingAssetData;
}

void FPrimaryAssetTypeData::RemoveAsset(FName AssetName, TMap<FSoftObjectPath, FPrimaryAssetId>& InOutReverseMap)
{
	FPrimaryAssetData ExistingAssetData;
	if (AssetMap.RemoveAndCopyValue(AssetName, ExistingAssetData))
	{
		const FSoftObjectPath& OldAssetRef = ExistingAssetData.AssetPtr.ToSoftObjectPath();
		if (!OldAssetRef.IsNull()) // Dynamic types can have null asset refs
		{
			InOutReverseMap.Remove(OldAssetRef);
		}
	}
	Info.NumberOfAssets = AssetMap.Num();
}

void FPrimaryAssetTypeData::RemoveAssets(TConstArrayView<FName> AssetNames, TMap<FSoftObjectPath, FPrimaryAssetId>& InOutReverseMap)
{
	FPrimaryAssetData ExistingAssetData;
	for (FName AssetName : AssetNames)
	{
		if (AssetMap.RemoveAndCopyValue(AssetName, ExistingAssetData))
		{
			const FSoftObjectPath& OldAssetRef = ExistingAssetData.AssetPtr.ToSoftObjectPath();
			if (!OldAssetRef.IsNull()) // Dynamic types can have null asset refs
			{
				InOutReverseMap.Remove(OldAssetRef);
			}
		}
	}
	Info.NumberOfAssets = AssetMap.Num();
}


void FPrimaryAssetTypeData::ResetAssets(TMap<FSoftObjectPath, FPrimaryAssetId>& InOutReverseMap)
{
	for (const TPair<FName, FPrimaryAssetData>& Pair : AssetMap)
	{
		const FPrimaryAssetData& AssetData = Pair.Value;
		const FSoftObjectPath& AssetRef = AssetData.AssetPtr.ToSoftObjectPath();
		if (!AssetRef.IsNull()) // Dynamic types can have null asset refs
		{
			InOutReverseMap.Remove(AssetRef);
		}
	}
	AssetMap.Reset();
	Info.NumberOfAssets = 0;
}

void FPrimaryAssetTypeData::ShrinkAssets()
{
	AssetMap.Shrink();
}

/** Version of rules with cached data */
struct FCompiledAssetManagerSearchRules : FAssetManagerSearchRules
{
	FCompiledAssetManagerSearchRules(const FAssetManagerSearchRules& InRules)
		: FAssetManagerSearchRules(InRules)
		, bShouldCallDelegate(InRules.ShouldIncludeDelegate.IsBound())
		, bShouldCheckWildcards(false)
	{
		for (const FString& String : IncludePatterns)
		{
			if (String.Len())
			{
				IncludeWildcards.Add(String);
				bShouldCheckWildcards = true;
			}
		}

		for (const FString& String : ExcludePatterns)
		{
			if (String.Len())
			{
				ExcludeWildcards.Add(String);
				bShouldCheckWildcards = true;
			}
		}

		// Check class first
		if (AssetBaseClass)
		{
			AssetClassNames.Add(AssetBaseClass->GetClassPathName());

#if WITH_EDITOR
			// Add any old names to the list in case things haven't been resaved
			TArray<FString> OldNames = FLinkerLoad::FindPreviousPathNamesForClass(AssetBaseClass->GetPathName(), false);
			AssetClassNames.Append(OldNames);
#endif
		}
	}

	bool PassesWildcardsAndDelegates(const FAssetData& AssetData, const UAssetManager* AssetManager) const
	{
		if (bShouldCheckWildcards)
		{
			// Check include and exclude patterns, will pass if no include patterns
			FString PackageString = AssetData.PackageName.ToString();
			for (const FString& Wildcard : ExcludeWildcards)
			{
				if (PackageString.MatchesWildcard(Wildcard, ESearchCase::IgnoreCase))
				{
					return false;
				}
			}

			bool bPassesInclude = (IncludeWildcards.Num() == 0);
			for (const FString& Wildcard : IncludeWildcards)
			{
				if (PackageString.MatchesWildcard(Wildcard, ESearchCase::IgnoreCase))
				{
					bPassesInclude = true;
					break;
				}
			}
			if (!bPassesInclude)
			{
				return false;
			}
		}

		if (!bSkipManagerIncludeCheck)
		{
			if (!AssetManager->ShouldIncludeInAssetSearch(AssetData, *this))
			{
				return false;
			}
		}

		if (bShouldCallDelegate)
		{
			if (!ShouldIncludeDelegate.Execute(AssetData, *this))
			{
				return false;
			}
		}

		return true;
	}

	TArray<FString> IncludeWildcards;
	TArray<FString> ExcludeWildcards;
	TArray<FTopLevelAssetPath> AssetClassNames;
	TSet<FTopLevelAssetPath> DerivedClassNames;
	bool bShouldCallDelegate;
	bool bShouldCheckWildcards;
};

const FPrimaryAssetType UAssetManager::MapType = FName("Map");
const FPrimaryAssetType UAssetManager::PrimaryAssetLabelType = FName("PrimaryAssetLabel");
const FPrimaryAssetType UAssetManager::PackageChunkType = FName("PackageChunk");
const FPrimaryAssetType UAssetManager::SuppressionType = FName("__Suppressed__");
const FString UAssetManager::AssetSearchRootsVirtualPath = TEXT("$AssetSearchRoots");
const FString UAssetManager::DynamicSearchRootsVirtualPath = TEXT("$DynamicSearchRoots");
FSimpleMulticastDelegate UAssetManager::OnCompletedInitialScanDelegate;
FSimpleMulticastDelegate UAssetManager::OnAssetManagerCreatedDelegate;

// Allow StartInitialLoading() bulk scan to continue past FinishInitialLoading() to cover:
// 1. Loading in subclass FinishInitialLoading() override after calling base FinishInitialLoading() 
// 2. Plugin loading in OnPostEngine callback, e.g. UDataRegistrySubsystem::LoadAllRegistries()
// Ideally a future AssetManager refactor will reduce update costs and remove the need for batching
// things up with bulk scanning.
static class FInitialBulkScanHelper
{
	UAssetManager* Scanner = nullptr;

public:
	void StartOnce(UAssetManager* Manager)
	{
		if (Scanner == nullptr)
		{
			Manager->PushBulkScanning();
			Scanner = Manager;
		}
	}

	void StopOnce()
	{
		if (Scanner)
		{
			Scanner->PopBulkScanning();
		}
		else
		{
			Scanner = reinterpret_cast<UAssetManager*>(8); // Avoid starting bulk scan after OnPostEngineInit
		}
	}
}
GInitialBulkScan;

// OnPostEngineInit fires in reverse order, register early to get a late callback
static FDelayedAutoRegisterHelper GInitialBulkScanStopper(EDelayedRegisterRunPhase::StartOfEnginePreInit,
	[] { FCoreDelegates::GetOnPostEngineInit().AddLambda([] { GInitialBulkScan.StopOnce(); }); } );

namespace UE::AssetManager::Private
{
	static constexpr int MaxDebugWarningLen = 1024;
	static constexpr int MaxEnsureMessageLen = 256;

	static TArray<FSoftObjectPath> GetPreloadPrimaryAssetsLoadList(
		const UAssetManager* Self, const TArray<FPrimaryAssetId>& AssetsToLoad, const TArray<FName>& LoadBundles, bool bLoadRecursive,
		FStringBuilderBase& DebugValid, FStringBuilderBase& DebugInvalid)
	{
		TArray<FSoftObjectPath> PathsToLoad;
		bool ExceededMaxLength = false;

		for (const FPrimaryAssetId& PrimaryAssetId : AssetsToLoad)
		{
			if (Self->GetPrimaryAssetLoadList(PathsToLoad, PrimaryAssetId, LoadBundles, bLoadRecursive, false))
			{
				if (DebugValid.Len() < MaxDebugWarningLen)
				{
					DebugValid << (DebugValid.Len() > 0 ? TEXT(", ") : TEXT("")) << PrimaryAssetId.ToString();
				}
				else
				{
					ExceededMaxLength = true;
				}
			}
			else
			{
				if (DebugInvalid.Len() < MaxDebugWarningLen)
				{
					DebugInvalid << (DebugInvalid.Len() > 0 ? TEXT(", ") : TEXT("")) << PrimaryAssetId.ToString();
				}
				else
				{
					ExceededMaxLength = true;
				}
			}
		}

		if (DebugInvalid.Len() > 0)
		{
			if (DebugValid.Len() > 0)
			{
				DebugInvalid << TEXT(" (Valid: ") << DebugValid << TEXT(")");
			}

			DebugInvalid << (ExceededMaxLength ? TEXT("...") : TEXT(""));

			UE_LOGF(LogAssetManager, Warning, "Requested preload of some Primary Assets failed: %ls", *DebugInvalid);

			// Trim for shorter ensure message
			int TrimEndLen = FMath::Max(0, DebugInvalid.Len() - MaxEnsureMessageLen);
			if (TrimEndLen > 0)
			{
				DebugInvalid.RemoveSuffix(TrimEndLen);
				DebugInvalid << TEXT("...");
			}
		}

		return PathsToLoad;
	}
}

UAssetManager::UAssetManager()
{
	bIsGlobalAsyncScanEnvironment = false;
	bShouldGuessTypeAndName = false;
	bShouldUseSynchronousLoad = false;
	bIsLoadingFromPakFiles = false;
	bShouldAcquireMissingChunksOnLoad = false;
	bTargetPlatformsAllowDevelopmentObjects = false;
	NumBulkScanRequests = 0;
	bIsManagementDatabaseCurrent = false;
	bIsPrimaryAssetDirectoryCurrent = false;
	bUpdateManagementDatabaseAfterScan = false;
	bIncludeOnlyOnDiskAssets = true;
	bHasCompletedInitialScan = false;
	NumberOfSpawnedNotifications = 0;
}

void UAssetManager::BeginDestroy()
{
	Super::BeginDestroy();

	// FPrimaryAssetTypeDatas need to have their AssetMaps cleared before their destructor,
	// because they have a contract that modifying AssetMap also modifies AssetPathMap.
	for (TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& Pair : AssetTypeMap)
	{
		Pair.Value->ResetAssets(AssetPathMap);
	}
}

void UAssetManager::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		const UAssetManagerSettings& Settings = GetSettings();
#if WITH_EDITOR
		IAssetRegistry& AssetRegistry = GetAssetRegistry();

		bIsGlobalAsyncScanEnvironment = AssetRegistry.IsSearchAllAssets();

		if (bIsGlobalAsyncScanEnvironment)
		{
			// Listen for when the asset registry has finished discovering files
			AssetRegistry.OnKnownGathersComplete().AddUObject(this, &UAssetManager::OnAssetRegistryFilesLoaded);
			AssetRegistry.OnInMemoryAssetCreated().AddUObject(this, &UAssetManager::OnInMemoryAssetCreated);
			AssetRegistry.OnInMemoryAssetDeleted().AddUObject(this, &UAssetManager::OnInMemoryAssetDeleted);
			AssetRegistry.OnAssetRenamed().AddUObject(this, &UAssetManager::OnAssetRenamed);
			AssetRegistry.OnAssetRemoved().AddUObject(this, &UAssetManager::OnAssetRemoved);
		}

		FEditorDelegates::PreBeginPIE.AddUObject(this, &UAssetManager::PreBeginPIE);
		FEditorDelegates::EndPIE.AddUObject(this, &UAssetManager::EndPIE);
		FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &UAssetManager::OnObjectPreSave);

		// In editor builds guess the type/name if allowed
		bShouldGuessTypeAndName = Settings.bShouldGuessTypeAndNameInEditor;
		bOnlyCookProductionAssets = Settings.bOnlyCookProductionAssets;

		// In editor builds, always allow asset registry searches for in-memory asset data, as that data can change when propagating AssetBundle tags post load.
		bIncludeOnlyOnDiskAssets = false;
#else 
		// Never guess type in cooked builds
		bShouldGuessTypeAndName = false;

		// Only cooked builds supoprt pak files and chunk download
		bIsLoadingFromPakFiles = FPlatformFileManager::Get().FindPlatformFile(TEXT("PakFile")) != nullptr;
		bShouldAcquireMissingChunksOnLoad = Settings.bShouldAcquireMissingChunksOnLoad;
#endif
		
		bShouldUseSynchronousLoad = IsRunningCommandlet();

		if (Settings.bShouldManagerDetermineTypeAndName)
		{
			FCoreUObjectDelegates::GetPrimaryAssetIdForObject.BindUObject(this, &UAssetManager::DeterminePrimaryAssetIdForObject);
		}

		LoadRedirectorMaps();

		StreamableManager.SetManagerName(FString::Printf(TEXT("%s.StreamableManager"), *GetPathName()));

		// Add /Game to initial list, games can add additional ones if desired
		AllAssetSearchRoots.Add(TEXT("/Game"));

		FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddUObject(this, &UAssetManager::CallPreGarbageCollect);
		
		TMap<FString, int32> PakChunkNameToChunkId;
		{
			TMap<int32, FString> ChunkIdStringOverride;
			GetPakChunkIdToStringMapping(ChunkIdStringOverride);
			PakChunkNameToChunkId.Reserve(ChunkIdStringOverride.Num());
			for (const TPair<int32, FString>& ChunkToString : ChunkIdStringOverride)
			{
				PakChunkNameToChunkId.Add(ChunkToString.Value, ChunkToString.Key);
			}
		}
		if (PakChunkNameToChunkId.Num())
		{
			UE_LOGF(LogAssetManager, Display, "Detected PakchunkId override config. Binding PakChunkId override delegate.");
			FCoreDelegates::PakChunkIdOverrideDelegate.BindLambda([LocalPakChunkNameToChunkId = MoveTemp(PakChunkNameToChunkId)](const FString& InPakchunkName)
				{
					return LocalPakChunkNameToChunkId.Contains(InPakchunkName) ? LocalPakChunkNameToChunkId[InPakchunkName] : INDEX_NONE;
				});
			if (FPakPlatformFile* Pak = (FPakPlatformFile*)FPlatformFileManager::Get().FindPlatformFile(TEXT("PakFile")))
			{
				Pak->RefreshPakChunkIndicies();
			}
		}
	}
}

void UAssetManager::GetCachedPrimaryAssetEncryptionKeyGuid(FPrimaryAssetId InPrimaryAssetId, FGuid& OutGuid)
{
	OutGuid.Invalidate();
	if (const FGuid* Guid = PrimaryAssetEncryptionKeyCache.Find(InPrimaryAssetId))
	{
		OutGuid = *Guid;
	}
}

bool UAssetManager::IsValid()
{
	return IsInitialized();
}

bool UAssetManager::IsInitialized()
{
	return GEngine && GEngine->AssetManager;
}

UAssetManager& UAssetManager::Get()
{
	UAssetManager* Singleton = GEngine->AssetManager;

	if (Singleton)
	{
		return *Singleton;
	}
	else
	{
		UE_LOGF(LogAssetManager, Fatal, "Cannot use AssetManager if no AssetManagerClassName is defined!");
		return *NewObject<UAssetManager>(); // never calls this
	}
}

UAssetManager* UAssetManager::GetIfValid()
{
	return GetIfInitialized();
}

UAssetManager* UAssetManager::GetIfInitialized()
{
	return GEngine ? GEngine->AssetManager : nullptr;
}

FPrimaryAssetId UAssetManager::CreatePrimaryAssetIdFromChunkId(int32 ChunkId)
{
	if (ChunkId == INDEX_NONE)
	{
		return FPrimaryAssetId();
	}

	// Name_0 is actually stored as 1 inside FName, so offset
	static FName ChunkName = "Chunk";
	return FPrimaryAssetId(PackageChunkType, FName(ChunkName, ChunkId + 1));
}

int32 UAssetManager::ExtractChunkIdFromPrimaryAssetId(const FPrimaryAssetId& PrimaryAssetId)
{
	if (PrimaryAssetId.PrimaryAssetType == PackageChunkType)
	{
		return PrimaryAssetId.PrimaryAssetName.GetNumber() - 1;
	}
	return INDEX_NONE;
}

IAssetRegistry& UAssetManager::GetAssetRegistry() const
{
	if (!CachedAssetRegistry)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		CachedAssetRegistry = &AssetRegistryModule.Get();
	}

	return *CachedAssetRegistry;
}

const UAssetManagerSettings& UAssetManager::GetSettings() const
{
	if (!CachedSettings)
	{
		CachedSettings = GetDefault<UAssetManagerSettings>();
	}
	return *CachedSettings;
}

FTimerManager* UAssetManager::GetTimerManager() const
{
#if WITH_EDITOR
	if (GEditor)
	{
		// In editor use the editor manager
		if (GEditor->IsTimerManagerValid())
		{
			return &GEditor->GetTimerManager().Get();
		}
	}
	else
#endif
	{
		// Otherwise we should always have a game instance
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& WorldContext : WorldContexts)
		{
			if (WorldContext.WorldType == EWorldType::Game && WorldContext.OwningGameInstance)
			{
				return &WorldContext.OwningGameInstance->GetTimerManager();
			}
		}
	}

	// This will only hit in very early startup
	return nullptr;
}

FPrimaryAssetId UAssetManager::DeterminePrimaryAssetIdForObject(const UObject* Object) const
{
	const UObject* AssetObject = Object;
	// First find the object that would be registered, need to use class if we're a BP CDO
	if (Object->HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetObject = Object->GetClass();
	}

	FString AssetPath = AssetObject->GetPathName();
	FPrimaryAssetId RegisteredId = GetPrimaryAssetIdForPath(FSoftObjectPath(AssetObject));

	if (RegisteredId.IsValid())
	{
		return RegisteredId;
	}

	FPrimaryAssetType FoundType;

	// Not registered, so search the types for one that matches class/path
	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		// Check the originally passed object, which is either an asset or a CDO, not the BP class
		if (Object->IsA(TypeData.Info.AssetBaseClassLoaded))
		{
			// Check paths, directories will end in /, specific paths will end in full assetname.assetname
			for (const FString& ScanPath : TypeData.RealAssetScanPaths)
			{
				if (AssetPath.StartsWith(ScanPath))
				{
					if (FoundType.IsValid())
					{
						UE_LOGF(LogAssetManager, Warning, "Found Duplicate PrimaryAssetType %ls for asset %ls which is already registered as %ls, it is not possible to have conflicting directories when bShouldManagerDetermineTypeAndName is true!", *TypeData.Info.PrimaryAssetType.ToString(), *AssetPath, *FoundType.ToString());
					}
					else
					{
						FoundType = TypeData.Info.PrimaryAssetType;
					}
				}
			}
		}
	}

	if (FoundType.IsValid())
	{
		// Use the package's short name, avoids issues with _C
		return FPrimaryAssetId(FoundType, FPackageName::GetShortFName(AssetObject->GetOutermost()->GetName()));
	}

	return FPrimaryAssetId();
}

bool UAssetManager::IsAssetDataBlueprintOfClassSet(const FAssetData& AssetData, const TSet<FTopLevelAssetPath>& ClassNameSet) const
{
	return UAssetRegistryHelpers::IsAssetDataBlueprintOfClassSet(AssetData, ClassNameSet);
}

int32 UAssetManager::SearchAssetRegistryPaths(TArray<FAssetData>& OutAssetDataList, const FAssetManagerSearchRules& Rules) const
{
	if (!Rules.AssetBaseClass)
	{
		return 0;
	}
	if (Rules.AssetScanPaths.IsEmpty())
	{
		return 0;
	}
	TArray<FString> Directories, PackageNames;
	TArray<FString> ScanPaths = Rules.AssetScanPaths;
	// Add path info
	if (!Rules.bSkipVirtualPathExpansion)
	{
		ExpandVirtualPaths(ScanPaths);
	}

	for (const FString& Path : ScanPaths)
	{
		int32 DotIndex = INDEX_NONE;
		if (Path.FindChar('.', DotIndex))
		{
			FString PackageName = Path.Mid(0, DotIndex); //avoid re-searching for index inside FPackageName::ObjectPathToPackageName

			PackageNames.AddUnique(MoveTemp(PackageName));
		}
		else if (Path.Len() > 0)
		{
			Directories.AddUnique(GetNormalizedPackagePath(Path, false));
		}
	}

#if WITH_EDITOR
	// Cooked data has the asset data already set up
	const bool bShouldDoSynchronousScan = !bIsGlobalAsyncScanEnvironment || Rules.bForceSynchronousScan;
	if (bShouldDoSynchronousScan)
	{
		ScanPathsSynchronous(ScanPaths);
	}
#endif
	
	FCompiledAssetManagerSearchRules CompiledRules(Rules);
	int32 InitialAssetCount = OutAssetDataList.Num();
	FARFilter ARFilter;
	IAssetRegistry& AssetRegistry = GetAssetRegistry();

	if (Rules.AssetBaseClass)
	{
		// Class check
		if (!Rules.bHasBlueprintClasses)
		{
			// Use class directly
			ARFilter.ClassPaths = CompiledRules.AssetClassNames;
			ARFilter.bRecursiveClasses = true;
		}
		else
		{
			// Search for all blueprints and then check derived classes later
			ARFilter.ClassPaths.Add(UBlueprintCore::StaticClass()->GetClassPathName());
			ARFilter.bRecursiveClasses = true;

			GetAssetRegistry().GetDerivedClassNames(CompiledRules.AssetClassNames, TSet<FTopLevelAssetPath>(), CompiledRules.DerivedClassNames);
		}
	}

	const bool bBothDirectoriesAndPackageNames = (Directories.Num() > 0 && PackageNames.Num() > 0);
	for (const FString& Directory : Directories)
	{
		ARFilter.PackagePaths.Add(FName(*Directory));
	}

	if (!bBothDirectoriesAndPackageNames)
	{
		// To get both the directories and package names we have to do two queries, since putting both in the same query only returns assets of those package names AND are in those directories.
		for (const FString& PackageName : PackageNames)
		{
			ARFilter.PackageNames.Add(FName(*PackageName));
		}
	}

	ARFilter.bRecursivePaths = true;
	ARFilter.bIncludeOnlyOnDiskAssets = !GIsEditor || IsRunningCookCommandlet(); // In editor check in memory, otherwise don't

	auto FilterLambda = [this, &OutAssetDataList, &CompiledRules](const FAssetData& AssetData)
	{
		// Verify blueprint class
		if (CompiledRules.bHasBlueprintClasses)
		{
			if (!IsAssetDataBlueprintOfClassSet(AssetData, CompiledRules.DerivedClassNames))
			{
				return true;
			}
		}

		if (!CompiledRules.PassesWildcardsAndDelegates(AssetData, this))
		{
			return true;
		}

		// Passed all filters
		OutAssetDataList.Add(AssetData);

		return true;
	};

	if (bBothDirectoriesAndPackageNames)
	{
		// To get both the directories and package names we have to do two queries, since putting both in the same query only returns assets of those package names AND are in those directories.
		AssetRegistry.EnumerateAssets(ARFilter, FilterLambda);

		for (const FString& PackageName : PackageNames)
		{
			ARFilter.PackageNames.Add(FName(*PackageName));
		}
		ARFilter.PackagePaths.Empty();
	}

	// Search asset registry and apply delegate, will always search entire registry
	AssetRegistry.EnumerateAssets(ARFilter, FilterLambda);

	return OutAssetDataList.Num() - InitialAssetCount;
}

bool UAssetManager::DoesAssetMatchSearchRules(const FAssetData& AssetData, const FAssetManagerSearchRules& Rules) const
{
	// This is slower than the scan version above when looking for many assets, but works on assets from any source
	FCompiledAssetManagerSearchRules CompiledRules(Rules);

	// Check class first
	if (Rules.AssetBaseClass)
	{
		GetAssetRegistry().GetDerivedClassNames(CompiledRules.AssetClassNames, TSet<FTopLevelAssetPath>(), CompiledRules.DerivedClassNames);

		if (!Rules.bHasBlueprintClasses)
		{
			if (!CompiledRules.DerivedClassNames.Contains(AssetData.AssetClassPath))
			{
				return false;
			}
		}
		else
		{
			if (!IsAssetDataBlueprintOfClassSet(AssetData, CompiledRules.DerivedClassNames))
			{
				return false;
			}
		}
	}

	TArray<FString> ScanPaths = Rules.AssetScanPaths;
	// Add path info
	if (!Rules.bSkipVirtualPathExpansion)
	{
		ExpandVirtualPaths(ScanPaths);
	}

	// If we have any scan paths, verify it's inside one
	bool bFoundPath = ScanPaths.Num() == 0;
	for (FString& Path : ScanPaths)
	{
		NormalizePackagePath(Path, false);
		if (AssetData.PackageName.ToString().StartsWith(Path))
		{
			bFoundPath = true;
			break;
		}
	}

	if (!bFoundPath)
	{
		return false;
	}

	if (!CompiledRules.PassesWildcardsAndDelegates(AssetData, this))
	{
		return false;
	}

	return true;
}

bool UAssetManager::ShouldIncludeInAssetSearch(const FAssetData& AssetData, const FAssetManagerSearchRules& SearchRules) const
{
	const UAssetManagerSettings& Settings = GetSettings();

	if (Settings.DirectoriesToExclude.Num() > 0)
	{
		// Check exclusion path, but only if we have paths as the package name string is slow to generate
		// Games can override this for more specific checks
		if (IsPathExcludedFromScan(AssetData.PackageName.ToString()))
		{
			return false;
		}
	}

	return true;
}

void UAssetManager::ScanPathsSynchronous(const TArray<FString>& PathsToScan) const
{
	TArray<FString> Directories;
	TArray<FString> PackageFilenames;

	for (const FString& Path : PathsToScan)
	{
		bool bAlreadyScanned = false;
		int32 DotIndex = INDEX_NONE;
		if (Path.FindChar('.', DotIndex))
		{
			FString PackageName = FPackageName::ObjectPathToPackageName(Path);

			for (const FString& AlreadyScanned : AlreadyScannedDirectories)
			{
				if (PackageName == AlreadyScanned || PackageName.StartsWith(AlreadyScanned + TEXT("/")))
				{
					bAlreadyScanned = true;
					break;
				}
			}

			if (!bAlreadyScanned)
			{
				FString AssetFilename;
				// Try both extensions
				if (FPackageName::TryConvertLongPackageNameToFilename(PackageName, AssetFilename, FPackageName::GetAssetPackageExtension()))
				{
					PackageFilenames.AddUnique(AssetFilename);
				}

				if (FPackageName::TryConvertLongPackageNameToFilename(PackageName, AssetFilename, FPackageName::GetMapPackageExtension()))
				{
					PackageFilenames.AddUnique(AssetFilename);
				}
			}
		}
		else
		{
			for (const FString& AlreadyScanned : AlreadyScannedDirectories)
			{
				if (Path == AlreadyScanned || Path.StartsWith(AlreadyScanned + TEXT("/")))
				{
					bAlreadyScanned = true;
					break;
				}
			}

			if (!bAlreadyScanned)
			{
				AlreadyScannedDirectories.Add(Path);

				// The asset registry currently crashes if you pass it either a ../ disk path or a /pluginname/ path that isn't mounted, so we need to verify the conversion would work but send the preconverted path
				FString OnDiskPath;

				if (FPackageName::TryConvertLongPackageNameToFilename(Path / TEXT(""), OnDiskPath))
				{
					Directories.AddUnique(Path);
				}
			}
		}
	}

	if (Directories.Num() > 0)
	{
		GetAssetRegistry().ScanPathsSynchronous(Directories);
	}
	if (PackageFilenames.Num() > 0)
	{
		GetAssetRegistry().ScanFilesSynchronous(PackageFilenames);
	}
}

int32 UAssetManager::ScanPathsForPrimaryAssets(FPrimaryAssetType PrimaryAssetType, const TArray<FString>& Paths, UClass* BaseClass, bool bHasBlueprintClasses, bool bIsEditorOnly, bool bForceSynchronousScan)
{
	LLM_SCOPE_BYTAG(AssetManager);
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetManager::ScanPathsForPrimaryAssets)

	if (bIsEditorOnly && !GIsEditor)
	{
		return 0;
	}

	TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	check(BaseClass);

	if (!FoundType)
	{
		UE_LOGF(LogAssetManager, VeryVerbose, "UAssetManager::ScanPathsForPrimaryAssets adding TypeData to AssetTypeMap: '%ls', (Paths=%ls%ls)",
			*PrimaryAssetType.ToString(),
			*FString::Join(TArrayView<const FString>(Paths).Left(FMath::Min(Paths.Num(), 3)), TEXT(", ")),
			(Paths.Num() > 3) ? TEXT(", ...") : TEXT(""));

		TSharedPtr<FPrimaryAssetTypeData> NewAsset = MakeShareable(new FPrimaryAssetTypeData(PrimaryAssetType, BaseClass, bHasBlueprintClasses, bIsEditorOnly));

		FoundType = &AssetTypeMap.Add(PrimaryAssetType, NewAsset.ToSharedRef());
	}

	// Should always be valid
	check(FoundType);

	FPrimaryAssetTypeData& TypeData = FoundType->Get();

	// Make sure types match
	if (TypeData.Info.AssetBaseClassLoaded != BaseClass || TypeData.Info.bHasBlueprintClasses != bHasBlueprintClasses || TypeData.Info.bIsEditorOnly != bIsEditorOnly)
	{
		auto CalculateEnsureMessage = [&TypeData, &Paths, BaseClass, bHasBlueprintClasses, bIsEditorOnly]()
		{
			FString Message;
			if (TypeData.Info.AssetBaseClassLoaded != BaseClass)
			{
				Message += FString::Printf(TEXT(", (BaseClass: '%s' != expected: '%s')"), *GetNameSafe(BaseClass), *GetNameSafe(TypeData.Info.AssetBaseClassLoaded.Get()));
			}

			if (TypeData.Info.bHasBlueprintClasses != bHasBlueprintClasses)
			{
				Message += FString::Printf(TEXT(", (bHasBlueprintClasses: '%d' != expected: '%d')"), bHasBlueprintClasses, TypeData.Info.bHasBlueprintClasses);
			}

			if (TypeData.Info.bIsEditorOnly != bIsEditorOnly)
			{
				Message += FString::Printf(TEXT(", (bIsEditorOnly: '%d' != expected: '%d')"), bIsEditorOnly, TypeData.Info.bIsEditorOnly);
			}

			if (Paths.Num() > 0)
			{
				// Context about where first PrimaryAssetType was set
				Message += FString::Printf(TEXT(", (Paths=%s%s)"),
					*FString::Join(TArrayView<const FString>(Paths).Left(FMath::Min(Paths.Num(), 3)), TEXT(", ")),
					(Paths.Num() > 3) ? TEXT(", ...") : TEXT(""));
			}

			Message += TEXT(". Another plugin, ini, or system has different parameters for this PrimaryAssetType name. Run with -LogCmds=\"LogAssetManager VeryVerbose\" to log more information.");

			return Message;
		};

		ensureMsgf(false, TEXT("UAssetManager::ScanPathsForPrimaryAssets TypeData parameters did not match PrimaryAssetType '%s'%s"), *TypeData.Info.PrimaryAssetType.ToString(), *CalculateEnsureMessage());
		return 0;
	}

	// Add path info
	for (const FString& Path : Paths)
	{
		InternalAddAssetScanPath(TypeData, Path);
	}

#if WITH_EDITOR
	// Cooked data has the asset data already set up
	const bool bShouldDoSynchronousScan = !bIsGlobalAsyncScanEnvironment || bForceSynchronousScan;
	if (!bShouldDoSynchronousScan && GetAssetRegistry().IsGathering())
	{
		// Keep track of the paths we asked for so once assets are discovered we will refresh the list
		for (const FString& Path : Paths)
		{
			TypeData.DeferredAssetScanPaths.Add(Path);
		}

		// Since we are still asynchronously discovering assets, we'll wait until that is done before populating with primary assets
		return 0;
	}
#endif

	FAssetManagerSearchRules SearchRules;
	SearchRules.AssetBaseClass = BaseClass;
	SearchRules.AssetScanPaths = Paths;
	SearchRules.bHasBlueprintClasses = bHasBlueprintClasses;
	SearchRules.bForceSynchronousScan = bForceSynchronousScan;	

	// Expand paths so we can record them for later
	ExpandVirtualPaths(SearchRules.AssetScanPaths);
	SearchRules.bSkipVirtualPathExpansion = true;
	for (const FString& Path : SearchRules.AssetScanPaths)
	{
		TypeData.RealAssetScanPaths.Add(Path);
	}

	TArray<FAssetData> AssetDataList;
	SearchAssetRegistryPaths(AssetDataList, SearchRules);

	int32 NumAdded = 0;
	// Now add to map or update as needed
	for (FAssetData& Data : AssetDataList)
	{
		if (!Data.IsTopLevelAsset())
		{
			// Only TopLevelAssets can be PrimaryAssets
			continue;
		}
		FPrimaryAssetId PrimaryAssetId = ExtractPrimaryAssetIdFromData(Data, PrimaryAssetType);

		// Remove invalid or wrong type assets
		if (!PrimaryAssetId.IsValid() || PrimaryAssetId.PrimaryAssetType == UAssetManager::SuppressionType ||
			PrimaryAssetId.PrimaryAssetType != PrimaryAssetType)
		{
			if (PrimaryAssetId.PrimaryAssetType == UAssetManager::SuppressionType)
			{
				// ExtractPrimaryAssetIdFromData decided to suppress this Asset from being a PrimaryAsset;
				// silently accept the suppression
			}
			else if (!PrimaryAssetId.IsValid())
			{
				UE_LOGF(LogAssetManager, Warning, "Ignoring primary asset %ls - PrimaryAssetType %ls - invalid primary asset ID", *Data.GetObjectPathString(), *PrimaryAssetType.ToString());
			}
			else
			{
				// Warn that 'Foo' conflicts with 'Bar', but only once per conflict
				static TSet<TPair<FPrimaryAssetType, FPrimaryAssetType>> IssuedWarnings;

				TTuple<FPrimaryAssetType, FPrimaryAssetType> ConflictPair(PrimaryAssetType, PrimaryAssetId.PrimaryAssetType);
				if (!IssuedWarnings.Contains(ConflictPair))
				{
					FString ConflictMsg = FString::Printf(TEXT("Ignoring PrimaryAssetType %s - Conflicts with %s - Asset: %s"), *PrimaryAssetType.ToString(), *PrimaryAssetId.PrimaryAssetType.ToString(), *Data.AssetName.ToString());

					UE_LOGF(LogAssetManager, Display, "%ls", *ConflictMsg);
					IssuedWarnings.Add(ConflictPair);
				}
			}
			continue;
		}

		NumAdded++;

		TryUpdateCachedAssetData(PrimaryAssetId, Data, false);
	}

	OnObjectReferenceListInvalidated();

	return NumAdded;
}

void UAssetManager::PushBulkScanning()
{
	if (++NumBulkScanRequests == 1)
	{
		StartBulkScanning();
	}
}

void UAssetManager::PopBulkScanning()
{
	ensure(NumBulkScanRequests > 0);
	if (--NumBulkScanRequests == 0)
	{
		StopBulkScanning();
	}
}

void UAssetManager::RemoveScanPathsForPrimaryAssets(FPrimaryAssetType PrimaryAssetType, const TArray<FString>& Paths, UClass* BaseClass, bool bHasBlueprintClasses, bool bIsEditorOnly /*= false*/)
{
	if (bIsEditorOnly && !GIsEditor)
	{
		return;
	}

	TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	check(BaseClass);

	if (!FoundType)
	{
		return;
	}

	FPrimaryAssetTypeData& TypeData = FoundType->Get();

	// Make sure types match
	if (!ensureMsgf(TypeData.Info.AssetBaseClassLoaded == BaseClass && TypeData.Info.bHasBlueprintClasses == bHasBlueprintClasses && TypeData.Info.bIsEditorOnly == bIsEditorOnly, TEXT("UAssetManager::RemoveScanPathsForPrimaryAssets TypeData parameters did not match for type '%s'"), *TypeData.Info.PrimaryAssetType.ToString()))
	{
		return;
	}

	TArray<FString> RemovedPaths;
	RemovedPaths.Reserve(Paths.Num());
	for (const FString& Path : Paths)
	{
		if (TypeData.Info.AssetScanPaths.Remove(Path))
		{
			RemovedPaths.Add(Path);
		}

		TypeData.DeferredAssetScanPaths.Remove(Path);
		TypeData.AdditionalAssetScanPaths.Remove(Path);
	}

	// Expand paths so we can record them for later
	ExpandVirtualPaths(RemovedPaths);
	for (const FString& Path : RemovedPaths)
	{
		TypeData.RealAssetScanPaths.Remove(Path);
	}
}

void UAssetManager::InternalAddAssetScanPath(FPrimaryAssetTypeData& TypeData, const FString& AssetScanPath)
{
	TypeData.Info.AssetScanPaths.AddUnique(AssetScanPath);

	if (!IsScanningFromInitialConfig())
	{
		TypeData.AdditionalAssetScanPaths.Add(AssetScanPath);
	}
}

void UAssetManager::RemovePrimaryAssetType(FPrimaryAssetType PrimaryAssetType)
{
	TSharedRef<FPrimaryAssetTypeData>* TypeDataRef;
	TypeDataRef = AssetTypeMap.Find(PrimaryAssetType);
	if (TypeDataRef)
	{
		TSharedPtr<FPrimaryAssetTypeData> TypeDataPtr = *TypeDataRef;
		AssetTypeMap.Remove(PrimaryAssetType);
		TypeDataPtr->ResetAssets(AssetPathMap);
		OnObjectReferenceListInvalidated();
	}
}

void UAssetManager::RemovePrimaryAssetsForTypeInMountPoint(FPrimaryAssetType PrimaryAssetType, FStringView RootPath)
{
	TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	if (!FoundType)
	{
		return;
	}

	FPrimaryAssetTypeData& TypeData = FoundType->Get();

	TArray<FName> AssetsToRemove;
	AssetsToRemove.Reserve(TypeData.GetAssets().Num());
	for (const TPair<FName, FPrimaryAssetData>& Pair : TypeData.GetAssets())
	{
		FNameBuilder PackageName(Pair.Value.GetAssetPtr().GetLongPackageFName());
		if (PackageName.ToView().StartsWith(RootPath))
		{
			AssetsToRemove.Add(Pair.Key);
		}
	}

	TypeData.RemoveAssets(AssetsToRemove, AssetPathMap);
	OnObjectReferenceListInvalidated();
}

void UAssetManager::StartBulkScanning()
{
	check(IsBulkScanning());

	NumberOfSpawnedNotifications = 0;
	bOldTemporaryCachingMode = GetAssetRegistry().GetTemporaryCachingMode();
	// Go into temporary caching mode to speed up class queries
	GetAssetRegistry().SetTemporaryCachingMode(true);
}

void UAssetManager::StopBulkScanning()
{
	check(!IsBulkScanning());

	GetAssetRegistry().SetTemporaryCachingMode(bOldTemporaryCachingMode);

	OnObjectReferenceListInvalidated();

	// Shrink some big containers because we've finished modifying them.
	AssetPathMap.Shrink();
	AssetTypeMap.Shrink();
	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& Pair : AssetTypeMap)
	{
		FPrimaryAssetTypeData& AssetTypeData = *Pair.Value;
		AssetTypeData.ShrinkAssets();
		AssetTypeData.RealAssetScanPaths.Shrink();
	}
}

bool UAssetManager::RegisterSpecificPrimaryAsset(const FPrimaryAssetId& PrimaryAssetId, const FAssetData& NewAssetData)
{
	if (!PrimaryAssetId.IsValid())
	{
		return false;
	}

	TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetId.PrimaryAssetType);
	if (!FoundType)
	{
		return false;
	}

	FPrimaryAssetTypeData& TypeData = FoundType->Get();
	if (!TryUpdateCachedAssetData(PrimaryAssetId, NewAssetData, false))
	{
		return false;
	}

	// Add to the list of scan paths so this will be found on refresh
	InternalAddAssetScanPath(TypeData, NewAssetData.GetSoftObjectPath().ToString());

	OnObjectReferenceListInvalidated();

	return true;
}

// This determines if we can use a faster conversion path
enum class EAssetDataCanBeSubobject { Yes, No };

template<EAssetDataCanBeSubobject ScanForSubobject>
bool TryToSoftObjectPath(const FAssetData& AssetData, FSoftObjectPath& OutSoftObjectPath);

bool UAssetManager::TryUpdateCachedAssetData(const FPrimaryAssetId& PrimaryAssetId, const FAssetData& NewAssetData, bool bAllowDuplicates)
{
	check(PrimaryAssetId.IsValid());

	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetId.PrimaryAssetType);

	if (!ensure(FoundType))
	{
		return false;
	}
	else
	{
		FPrimaryAssetTypeData& TypeData = FoundType->Get();

		const FPrimaryAssetData* OldData = TypeData.GetAssets().Find(PrimaryAssetId.PrimaryAssetName);

		FSoftObjectPath NewAssetPath;
		if (!TryToSoftObjectPath<EAssetDataCanBeSubobject::No>(NewAssetData, NewAssetPath))
		{
			UE_LOGF(LogAssetManager, Warning, "Tried to add primary asset %ls, but it is not a TopLevelAsset and so cannot be a primary asset",
				*NewAssetData.GetObjectPathString())
			return false;
		}

		ensure(NewAssetPath.IsAsset());

		if (OldData && OldData->GetAssetPtr().ToSoftObjectPath() != NewAssetPath)
		{
			UE_LOGF(LogAssetManager, Warning, 
				"Found duplicate PrimaryAssetID %ls, path %ls conflicts with existing path %ls. Two different primary assets can not have the same type and name.", 
				*PrimaryAssetId.ToString(), *OldData->GetAssetPtr().ToString(), *NewAssetPath.ToString());

#if WITH_EDITOR
			if (GIsEditor)
			{
				const int MaxNotificationsPerFrame = 5;
				if (NumberOfSpawnedNotifications++ < MaxNotificationsPerFrame)
				{
					FNotificationInfo Info(FText::Format(LOCTEXT("DuplicateAssetId", "Duplicate Asset ID {0} used by both {1} and {2}, rename one to avoid a conflict."),
						FText::FromString(PrimaryAssetId.ToString()),
						FText::FromString(OldData->GetAssetPtr().ToSoftObjectPath().GetLongPackageName()),
						FText::FromString(NewAssetPath.GetLongPackageName())));
					Info.ExpireDuration = 30.0f;

					TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
					if (Notification.IsValid())
					{
						Notification->SetCompletionState(SNotificationItem::CS_Fail);
					}
				}
			}
			else
#endif
			// Don't ensure for editor only types as they will not cause an actual game problem
			if (!bAllowDuplicates && !TypeData.Info.bIsEditorOnly)
			{
				ensureMsgf(!OldData, TEXT("Found Duplicate PrimaryAssetID %s! Path %s conflicts with existing path %s"),
					*PrimaryAssetId.ToString(), *OldData->GetAssetPtr().ToString(), *NewAssetPath.ToString());
			}
		}

		// Update data and path, don't touch state or references
		FSoftObjectPath NewARLookupPath = NewAssetData.GetSoftObjectPath(); // This will not have _C
		FSoftObjectPtr NewAssetPtr = FSoftObjectPtr(NewAssetPath); // This will have _C
		TypeData.FindOrAddAsset(PrimaryAssetId.PrimaryAssetName, NewAssetPtr, NewARLookupPath, AssetPathMap);

		// If the types don't match, update the registry
		IAssetRegistry& LocalAssetRegistry = GetAssetRegistry();
		FPrimaryAssetId SavedId = NewAssetData.GetPrimaryAssetId();
		FPrimaryAssetId ObjectPathId = LocalAssetRegistry.GetAssetByObjectPath(NewARLookupPath, true).GetPrimaryAssetId();
		if (SavedId != PrimaryAssetId || (ObjectPathId.IsValid() && SavedId != ObjectPathId))
		{
			LocalAssetRegistry.SetPrimaryAssetIdForObjectPath(NewARLookupPath, PrimaryAssetId);
		}

		if (NewAssetData.TaggedAssetBundles)
		{
			CachedAssetBundles.Add(PrimaryAssetId, NewAssetData.TaggedAssetBundles);

#if WITH_EDITOR
			if (GIsEditor)
			{
				// Add the bundles to our data that notifies the cooker of extra soft references from
				// primary assets, if the cooker encounters those primary assets referenced from elsewhere
				// rather than us reporting them as AlwaysCook in ModifyCook
				// Ignore the links from editor-only TypeDatas; the cooker only cares about the UsedInGame links.
				if (!TypeData.Info.bIsEditorOnly)
				{
					TArray<FTopLevelAssetPath>& Paths = AssetBundlePathsForPackage.FindOrAdd(NewAssetData.PackageName);
					Paths.Empty();
					for (const FAssetBundleEntry& Entry : NewAssetData.TaggedAssetBundles->Bundles)
					{
						Paths.Append(Entry.AssetPaths);
					}
					Paths.Sort([](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) { return A.CompareFast(B) < 0; });
					Paths.SetNum(Algo::Unique(Paths));
				}
			}
#endif
		}
		else if (OldData)
		{
			CachedAssetBundles.Remove(PrimaryAssetId);
		}
	}
	return true;
}

int32 UAssetManager::ScanPathForPrimaryAssets(FPrimaryAssetType PrimaryAssetType, const FString& Path, UClass* BaseClass, bool bHasBlueprintClasses, bool bIsEditorOnly, bool bForceSynchronousScan)
{
	return ScanPathsForPrimaryAssets(PrimaryAssetType, TArray<FString>{Path}, BaseClass, bHasBlueprintClasses, bIsEditorOnly, bForceSynchronousScan);
}

bool UAssetManager::AddDynamicAsset(const FPrimaryAssetId& PrimaryAssetId, const FSoftObjectPath& AssetPath, const FAssetBundleData& BundleData)
{
	if (!ensure(PrimaryAssetId.IsValid()))
	{
		return false;
	}

	if (!ensure(AssetPath.IsNull() || AssetPath.IsAsset()))
	{
		return false;
	}

	FPrimaryAssetType PrimaryAssetType = PrimaryAssetId.PrimaryAssetType;
	TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	if (!FoundType)
	{
		TSharedPtr<FPrimaryAssetTypeData> NewAsset = MakeShareable(new FPrimaryAssetTypeData());
		NewAsset->Info.PrimaryAssetType = PrimaryAssetType;
		NewAsset->Info.bIsDynamicAsset = true;

		FoundType = &AssetTypeMap.Add(PrimaryAssetType, NewAsset.ToSharedRef());
	}

	// Should always be valid
	check(FoundType);

	FPrimaryAssetTypeData& TypeData = FoundType->Get();

	// This needs to be a dynamic type, types cannot be both dynamic and loaded off disk
	if (!ensure(TypeData.Info.bIsDynamicAsset))
	{
		return false;
	}

	const FPrimaryAssetData* OldData = TypeData.GetAssets().Find(PrimaryAssetId.PrimaryAssetName);
	if (OldData && OldData->GetAssetPtr().ToSoftObjectPath() != AssetPath)
	{
		UE_LOGF(LogAssetManager, Warning, "AddDynamicAsset on %ls called with conflicting path. Path %ls is replacing path %ls",
			*PrimaryAssetId.ToString(), *OldData->GetAssetPtr().ToString(), *AssetPath.ToString());
	}

	if (BundleData.Bundles.Num() > 0)
	{
		TypeData.FindOrAddAsset(PrimaryAssetId.PrimaryAssetName, FSoftObjectPtr(AssetPath), AssetPath, AssetPathMap);
		CachedAssetBundles.Add(PrimaryAssetId, MakeShared<FAssetBundleData, ESPMode::ThreadSafe>(BundleData));
	}
	else if (OldData)
	{
		CachedAssetBundles.Remove(PrimaryAssetId);
		TypeData.RemoveAsset(PrimaryAssetId.PrimaryAssetName, AssetPathMap);
	}

	return true;
}

void UAssetManager::RecursivelyExpandBundleData(FAssetBundleData& BundleData) const
{
	TArray<FTopLevelAssetPath> ReferencesToExpand;
	TSet<FName> FoundBundleNames;

	for (const FAssetBundleEntry& Entry : BundleData.Bundles)
	{
		FoundBundleNames.Add(Entry.BundleName);

		for (const FTopLevelAssetPath& Reference : Entry.AssetPaths)
		{
			ReferencesToExpand.AddUnique(Reference);
		}
	}

	// Expandable references can increase recursively
	for (int32 i = 0; i < ReferencesToExpand.Num(); i++)
	{
		FPrimaryAssetId FoundId = GetPrimaryAssetIdForPath(FSoftObjectPath(ReferencesToExpand[i]));
		TArray<FAssetBundleEntry> FoundEntries;

		if (FoundId.IsValid() && GetAssetBundleEntries(FoundId, FoundEntries))
		{
			for (const FAssetBundleEntry& FoundEntry : FoundEntries)
			{
				// Make sure the bundle name matches
				if (FoundBundleNames.Contains(FoundEntry.BundleName))
				{
					BundleData.AddBundleAssets(FoundEntry.BundleName, FoundEntry.AssetPaths);

					for (const FTopLevelAssetPath& FoundReference : FoundEntry.AssetPaths)
					{
						// Keep recursing
						ReferencesToExpand.AddUnique(FoundReference);
					}
				}
			}
		}
	}
}

void UAssetManager::SetPrimaryAssetTypeRules(FPrimaryAssetType PrimaryAssetType, const FPrimaryAssetRules& Rules)
{
	// Can't set until it's been scanned at least once
	TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	if (ensure(FoundType))
	{
		(*FoundType)->Info.Rules = Rules;
	}
}

void UAssetManager::SetPrimaryAssetRules(FPrimaryAssetId PrimaryAssetId, const FPrimaryAssetRules& Rules)
{
	static FPrimaryAssetRules DefaultRules;
	LLM_SCOPE_BYTAG(AssetManager);

	FPrimaryAssetRulesExplicitOverride ExplicitRules;
	ExplicitRules.Rules = Rules;
	ExplicitRules.bOverridePriority = (Rules.Priority != DefaultRules.Priority);
	ExplicitRules.bOverrideApplyRecursively = (Rules.bApplyRecursively != DefaultRules.bApplyRecursively);
	ExplicitRules.bOverrideChunkId = (Rules.ChunkId != DefaultRules.ChunkId);
	ExplicitRules.bOverrideCookRule = (Rules.CookRule != DefaultRules.CookRule);
	
	SetPrimaryAssetRulesExplicitly(PrimaryAssetId, ExplicitRules);
}

void UAssetManager::SetPrimaryAssetRulesExplicitly(FPrimaryAssetId PrimaryAssetId, const FPrimaryAssetRulesExplicitOverride& ExplicitRules)
{
	if (!ExplicitRules.HasAnyOverride())
	{
		AssetRuleOverrides.Remove(PrimaryAssetId);
	}
	else
	{
		if (!GIsEditor && AssetRuleOverrides.Find(PrimaryAssetId))
		{
			UE_LOGF(LogAssetManager, Error, "Duplicate Rule overrides found for asset %ls!", *PrimaryAssetId.ToString());
		}

		AssetRuleOverrides.Add(PrimaryAssetId, ExplicitRules);
	}
	bIsManagementDatabaseCurrent = false;
}

FPrimaryAssetRules UAssetManager::GetPrimaryAssetRules(FPrimaryAssetId PrimaryAssetId) const
{
	FPrimaryAssetRules Result;

	// Allow setting management rules before scanning
	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetId.PrimaryAssetType);

	if (FoundType)
	{
		Result = (*FoundType)->Info.Rules;

		// Selectively override
		const FPrimaryAssetRulesExplicitOverride* FoundRulesOverride = AssetRuleOverrides.Find(PrimaryAssetId);

		if (FoundRulesOverride)
		{
			FoundRulesOverride->OverrideRulesExplicitly(Result);
		}

		if (Result.Priority < 0)
		{
			// Make sure it's at least 1
			Result.Priority = 1;
		}
	}

	return Result;
}

bool UAssetManager::GetPrimaryAssetData(const FPrimaryAssetId& PrimaryAssetId, FAssetData& AssetData) const
{
	const FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);

	if (NameData)
	{
		FAssetData CachedAssetData = GetAssetRegistry().GetAssetByObjectPath(NameData->GetARLookupPath(), bIncludeOnlyOnDiskAssets);

		if (CachedAssetData.IsValid())
		{
			AssetData = MoveTemp(CachedAssetData);
			return true;
		}
	}
	return false;
}

bool UAssetManager::GetPrimaryAssetDataList(FPrimaryAssetType PrimaryAssetType, TArray<FAssetData>& AssetDataList) const
{
	IAssetRegistry& Registry = GetAssetRegistry();
	bool bAdded = false;
	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	if (FoundType)
	{
		const FPrimaryAssetTypeData& TypeData = FoundType->Get();

		for (const TPair<FName, FPrimaryAssetData>& Pair : TypeData.GetAssets())
		{
			FAssetData CachedAssetData = Registry.GetAssetByObjectPath(Pair.Value.GetARLookupPath(), bIncludeOnlyOnDiskAssets);

			if (CachedAssetData.IsValid())
			{
				bAdded = true;
				AssetDataList.Add(MoveTemp(CachedAssetData));
			}
		}
	}

	return bAdded;
}

UObject* UAssetManager::GetPrimaryAssetObject(const FPrimaryAssetId& PrimaryAssetId) const
{
	const FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);

	if (NameData)
	{
		return NameData->GetAssetPtr().Get();
	}

	return nullptr;
}

bool UAssetManager::GetPrimaryAssetObjectList(FPrimaryAssetType PrimaryAssetType, TArray<UObject*>& ObjectList) const
{
	bool bAdded = false;
	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	if (FoundType)
	{
		const FPrimaryAssetTypeData& TypeData = FoundType->Get();

		for (const TPair<FName, FPrimaryAssetData>& Pair : TypeData.GetAssets())
		{
			UObject* FoundObject = Pair.Value.GetAssetPtr().Get();

			if (FoundObject)
			{
				ObjectList.Add(FoundObject);
				bAdded = true;
			}
		}
	}

	return bAdded;
}

FSoftObjectPath UAssetManager::GetPrimaryAssetPath(const FPrimaryAssetId& PrimaryAssetId) const
{
	const FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);

	if (NameData)
	{
		return NameData->GetAssetPtr().ToSoftObjectPath();
	}
	return FSoftObjectPath();
}

bool UAssetManager::GetPrimaryAssetPathList(FPrimaryAssetType PrimaryAssetType, TArray<FSoftObjectPath>& AssetPathList) const
{
	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	if (FoundType)
	{
		const FPrimaryAssetTypeData& TypeData = FoundType->Get();

		for (const TPair<FName, FPrimaryAssetData>& Pair : TypeData.GetAssets())
		{
			const FSoftObjectPtr& AssetPtr = Pair.Value.GetAssetPtr();
			if (!AssetPtr.IsNull())
			{
				AssetPathList.AddUnique(AssetPtr.ToSoftObjectPath());
			}
		}
	}

	return AssetPathList.Num() > 0;
}

FPrimaryAssetId UAssetManager::GetPrimaryAssetIdForObject(UObject* Object) const
{
	// Use path instead of calling on Object, we only want it if it's registered
	return GetPrimaryAssetIdForPath(FSoftObjectPath(Object));
}

FPrimaryAssetId UAssetManager::GetPrimaryAssetIdForData(const FAssetData& AssetData) const
{
	return GetPrimaryAssetIdForPath(GetAssetPathForData(AssetData));
}

FPrimaryAssetId UAssetManager::GetPrimaryAssetIdForPath(const FSoftObjectPath& ObjectPath) const
{
	const FPrimaryAssetId* FoundIdentifier = AssetPathMap.Find(ObjectPath);

	// Check redirector list
	if (!FoundIdentifier)
	{
		FSoftObjectPath RedirectedPath = GetRedirectedAssetPath(ObjectPath);

		if (!RedirectedPath.IsNull())
		{
			FoundIdentifier = AssetPathMap.Find(RedirectedPath);
		}
	}

	if (FoundIdentifier)
	{
		return *FoundIdentifier;
	}

	return FPrimaryAssetId();
}

FPrimaryAssetId UAssetManager::GetPrimaryAssetIdForPackage(FName PackagePath) const
{
	FString PackageString = PackagePath.ToString();
	FString AssetName = FPackageName::GetShortName(PackageString);

	FPrimaryAssetId FoundId;
	FSoftObjectPath PossibleAssetPath(FString::Printf(TEXT("%s.%s"), *PackageString, *AssetName));

	// Try without _C first
	if (PossibleAssetPath.IsValid())
	{
		FoundId = GetPrimaryAssetIdForPath(PossibleAssetPath);

		if (FoundId.IsValid())
		{
			return FoundId;
		}
	}

	// Then try _C
	PossibleAssetPath = FSoftObjectPath(FString::Printf(TEXT("%s.%s_C"), *PackageString, *AssetName));

	if (PossibleAssetPath.IsValid())
	{
		FoundId = GetPrimaryAssetIdForPath(PossibleAssetPath);
	}

	return FoundId;
}

FPrimaryAssetId UAssetManager::ExtractPrimaryAssetIdFromData(const FAssetData& AssetData, FPrimaryAssetType SuggestedType) const
{
	FPrimaryAssetId FoundId = AssetData.GetPrimaryAssetId();

	if (!FoundId.IsValid() && SuggestedType.IsValid())
	{
		if (AssetData.PackageFlags & PKG_ContainsMap)
		{
			// UE-216073: Temporary solution to turn off WorldPartition _Generated_ streaming cells packages
			// from being PrimaryAssets, since doing so causes duplicate PrimaryAssetId with their owning map.
			// TODO: Fix this properly by adding a class-specific hook on the PrimaryAssetType's baseclass to make
			// the decision for what the InAssetName should be for the asset's PrimaryAssetId.
			if (FPackageName::IsUnderGeneratedPackageSubPath(WriteToString<256>(AssetData.PackageName)))
			{
				return FPrimaryAssetId(UAssetManager::SuppressionType, AssetData.PackageName);
			}
		}

		FName SuggestedAssetName = AssetData.AssetName;
		if (SuggestedType == MapType)
		{
			// Maps use the full package name, this is a hack that only works properly with the default Map type defined in BaseGame.ini
			SuggestedAssetName = AssetData.PackageName;
		}

		if (bShouldGuessTypeAndName)
		{
			const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(SuggestedType);

			if (ensure(FoundType))
			{
				// If asset at this path is already known about return that
				FPrimaryAssetId OldID = GetPrimaryAssetIdForPath(GetAssetPathForData(AssetData));

				if (OldID.IsValid())
				{
					return OldID;
				}

				return FPrimaryAssetId(SuggestedType, SuggestedAssetName);
			}
		}
	}

	return FoundId;
}

bool UAssetManager::GetPrimaryAssetIdList(FPrimaryAssetType PrimaryAssetType, TArray<FPrimaryAssetId>& PrimaryAssetIdList, EAssetManagerFilter Filter) const
{
	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	if (FoundType)
	{
		const FPrimaryAssetTypeData& TypeData = FoundType->Get();

		for (const TPair<FName, FPrimaryAssetData>& Pair : TypeData.GetAssets())
		{
			if ((!(Filter & EAssetManagerFilter::UnloadedOnly)) || ((Pair.Value.CurrentState.GetBundleNames().Num() == 0) && (Pair.Value.PendingState.GetBundleNames().Num() == 0)))
			{
				PrimaryAssetIdList.Add(FPrimaryAssetId(PrimaryAssetType, Pair.Key));
			}
		}
	}

	return PrimaryAssetIdList.Num() > 0;
}

bool UAssetManager::GetPrimaryAssetTypeInfo(FPrimaryAssetType PrimaryAssetType, FPrimaryAssetTypeInfo& AssetTypeInfo) const
{
	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);

	if (FoundType)
	{
		AssetTypeInfo = (*FoundType)->Info;

		return true;
	}

	return false;
}

void UAssetManager::GetPrimaryAssetTypeInfoList(TArray<FPrimaryAssetTypeInfo>& AssetTypeInfoList) const
{
	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		AssetTypeInfoList.Add(TypeData.Info);
	}
}

void UAssetManager::GetPrimaryAssetsInPath(FPrimaryAssetType PrimaryAssetType, FStringView Path, TArray<FPrimaryAssetId>& PrimaryAssetIdList) const
{
	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetType);
	if (FoundType)
	{
		const FPrimaryAssetTypeData& TypeData = FoundType->Get();
		for (const TPair<FName, FPrimaryAssetData>& Pair : TypeData.GetAssets())
		{
			FNameBuilder PackageName(Pair.Value.GetAssetPtr().GetLongPackageFName());
			if (PackageName.ToView().StartsWith(Path))
			{
				PrimaryAssetIdList.Add(FPrimaryAssetId(PrimaryAssetType, Pair.Key));
			}
		}
	}
}

TSharedPtr<FStreamableHandle> UAssetManager::ChangeBundleStateForPrimaryAssets(
	const TArray<FPrimaryAssetId>& AssetsToChange,
	const TArray<FName>& AddBundles,
	const TArray<FName>& RemoveBundles,
	bool bRemoveAllBundles,
	FAssetManagerLoadParams&& LoadParams,
	UE::FSourceLocation Location)
{
	TArray<TSharedPtr<FStreamableHandle> > AllHandles;
	int32 MoviePlayerNumAssets = 0;
	for (const FPrimaryAssetId& PrimaryAssetId : AssetsToChange)
	{
		MoviePlayerNumAssets++;
		// Call the blocking tick every 500 assets.
		if (MoviePlayerNumAssets >= 500)
		{
			MoviePlayerNumAssets = 0;
			FMoviePlayerProxy::BlockingTick();
		}

		FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);

		if (NameData)
		{
			FPlatformMisc::PumpEssentialAppMessages();

			// Iterate list of changes, compute new bundle set

			// Use pending state if valid
			TArray<FName, TInlineAllocator<32>> NewBundleState;

			if (!bRemoveAllBundles)
			{
				NewBundleState = NameData->PendingState.IsValid() ? NameData->PendingState.GetBundleNames() : NameData->CurrentState.GetBundleNames();

				for (const FName& RemoveBundle : RemoveBundles)
				{
					NewBundleState.Remove(RemoveBundle);
				}
			}

			for (const FName& AddBundle : AddBundles)
			{
				NewBundleState.AddUnique(AddBundle);
			}

			NewBundleState.Sort(FNameLexicalLess());

			// If the pending state is valid, check if it is different
			if (NameData->PendingState.IsValid())
			{
				if (NameData->PendingState.IsSame(NewBundleState, LoadParams.DownloadParams))
				{
					// This will wait on any existing handles to finish
					AllHandles.Add(NameData->PendingState.GetHandle());
					continue;
				}

				// Clear pending state
				constexpr const bool bCancelHandle = true;
				NameData->PendingState.Reset(bCancelHandle);
			}
			else if (NameData->CurrentState.IsValid() && NameData->CurrentState.IsSame(NewBundleState, LoadParams.DownloadParams))
			{
				// If no pending, compare with current
				continue;
			}

			TArray<FSoftObjectPath> PathsToLoad;

			// Gather asset refs
			const FSoftObjectPath& AssetPath = NameData->GetAssetPtr().ToSoftObjectPath();

			if (!AssetPath.IsNull())
			{
				// Dynamic types can have no base asset path
				PathsToLoad.Add(AssetPath);
			}

			for (const FName& BundleName : NewBundleState)
			{
				FAssetBundleEntry Entry = GetAssetBundleEntry(PrimaryAssetId, BundleName);

				if (Entry.IsValid())
				{
					for (const FTopLevelAssetPath& Path : Entry.AssetPaths)
					{
						PathsToLoad.AddUnique(FSoftObjectPath(Path));
					}
				}
				else
				{
					UE_LOGF(LogAssetManager, Verbose, "ChangeBundleStateForPrimaryAssets: No assets for bundle %ls::%ls", *PrimaryAssetId.ToString(), *BundleName.ToString());
				}
			}

			if (PathsToLoad.Num() == 0)
			{
				// New state has no assets to load. Set the CurrentState's bundles and clear the handle
				NameData->CurrentState.SetState({}, NewBundleState, LoadParams.DownloadParams);
				continue;
			}

			TStringBuilder<1024> DebugName;
			PrimaryAssetId.AppendString(DebugName);

			if (NewBundleState.Num() > 0)
			{
				DebugName << TEXT(" (");

				for (int32 Index = 0; Index < NewBundleState.Num(); Index++)
				{
					if (Index > 0)
					{
						DebugName << TEXT(", ");
					}
					NewBundleState[Index].AppendString(DebugName);
				}

				DebugName << TEXT(")");
			}

			FAssetManagerLoadParams AssetListLoadParams;
			AssetListLoadParams.Priority = LoadParams.Priority;
			AssetListLoadParams.DownloadParams = CopyTemp(LoadParams.DownloadParams);
			
			TSharedPtr<FStreamableHandle> NewHandle = LoadAssetList(MoveTemp(PathsToLoad), MoveTemp(AssetListLoadParams), FString(DebugName), CopyTemp(Location));
			if (NewHandle && !NewHandle->HasLoadCompleted())
			{
				// save handle so possible duplicate requests will not override this one
				// set only if loading is not completed yet because completion callback might already reset the PendingState

				// OnAssetStateChangeCompleted requires this data to handle state transition correctly.
				NameData->PendingState.SetState(NewHandle, NewBundleState, LoadParams.DownloadParams);

				NewHandle->BindCompleteDelegate(FStreamableDelegateWithHandle::CreateUObject(this, &UAssetManager::OnAssetStateChangeCompleted, PrimaryAssetId));
				NewHandle->BindCancelDelegate(FStreamableDelegateWithHandle::CreateUObject(this, &UAssetManager::OnAssetStateChangeCompleted, PrimaryAssetId));
			}
			else
			{
				// OnAssetStateChangeCompleted requires this data to handle state transition correctly.
				NameData->PendingState.SetState({}, NewBundleState, LoadParams.DownloadParams);

				OnAssetStateChangeCompleted(NewHandle, PrimaryAssetId);
			}

			if (!NewHandle.IsValid())
			{
				// LoadAssetList already logs an error, no need to do it here as well
				continue;
			}

			AllHandles.Add(NewHandle);
		}
		else
		{
			WarnAboutInvalidPrimaryAsset(PrimaryAssetId, TEXT("ChangeBundleStateForPrimaryAssets failed to find NameData"));
		}
	}

	FStreamableAsyncLoadParams CombinedHandleParams;
	CombinedHandleParams.OnComplete = MoveTemp(LoadParams.OnComplete);
	CombinedHandleParams.OnCancel = MoveTemp(LoadParams.OnCancel);
	CombinedHandleParams.OnUpdate = MoveTemp(LoadParams.OnUpdate);

	return StreamableManager.CreateCombinedHandle(
		AllHandles,
		FStreamableHandle::HandleDebugName_CombinedHandle,
		EStreamableManagerCombinedHandleOptions::None,
		MoveTemp(CombinedHandleParams),
		CopyTemp(Location));
}

TSharedPtr<FStreamableHandle> UAssetManager::ChangeBundleStateForMatchingPrimaryAssets(const TArray<FName>& NewBundles, const TArray<FName>& OldBundles, FAssetManagerLoadParams&& LoadParams, UE::FSourceLocation Location)
{
	TArray<FPrimaryAssetId> AssetsToChange;

	if (GetPrimaryAssetsWithBundleState(AssetsToChange, TArray<FPrimaryAssetType>(), OldBundles))
	{
		// This will call delegate when done
		return ChangeBundleStateForPrimaryAssets(AssetsToChange, NewBundles, OldBundles, false, MoveTemp(LoadParams), MoveTemp(Location));
	}

	// Nothing to transition, call delegate now
	FStreamableHandle::ExecuteDelegate(MoveTemp(LoadParams.OnComplete));
	return nullptr;
}

bool UAssetManager::GetPrimaryAssetLoadSet(TSet<FSoftObjectPath>& OutAssetLoadSet, const FPrimaryAssetId& PrimaryAssetId, const TArray<FName>& LoadBundles, bool bLoadRecursive) const
{
	TArray<FSoftObjectPath> Array;
	bool bReturnValue = GetPrimaryAssetLoadList(Array, PrimaryAssetId, LoadBundles, bLoadRecursive, false);
	OutAssetLoadSet.Append(Array);
	return bReturnValue;
}

bool UAssetManager::GetPrimaryAssetLoadList(TArray<FSoftObjectPath>& OutAssetLoadList, const FPrimaryAssetId& PrimaryAssetId, const TArray<FName>& LoadBundles, bool bLoadRecursive, bool bEnsureUniqueness) const
{
	const FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);
	if (NameData)
	{
		// Gather asset refs
		const FSoftObjectPath& AssetPath = NameData->GetAssetPtr().ToSoftObjectPath();
		if (!AssetPath.IsNull())
		{
			// Dynamic types can have no base asset path
			if (bEnsureUniqueness)
			{
				OutAssetLoadList.AddUnique(AssetPath);
			}
			else
			{
				OutAssetLoadList.Add(AssetPath);
			}
		}

		// Construct a temporary bundle data with the bundles specified
		FAssetBundleData TempBundleData;
		for (const FName& BundleName : LoadBundles)
		{
			FAssetBundleEntry Entry = GetAssetBundleEntry(PrimaryAssetId, BundleName);

			if (Entry.IsValid())
			{
				TempBundleData.Bundles.Add(Entry);
			}
		}

		if (bLoadRecursive)
		{
			RecursivelyExpandBundleData(TempBundleData);
		}

		for (const FAssetBundleEntry& Entry : TempBundleData.Bundles)
		{
			for (const FTopLevelAssetPath& Path : Entry.AssetPaths)
			{
				if (bEnsureUniqueness)
				{
					OutAssetLoadList.AddUnique(FSoftObjectPath(Path));
				}
				else
				{
					OutAssetLoadList.Add(FSoftObjectPath(Path));
				}
			}
		}
	}
	else
	{
		WarnAboutInvalidPrimaryAsset(PrimaryAssetId, TEXT("GetPrimaryAssetLoadList failed to find NameData"));
	}
	return NameData != nullptr;
}

TSharedPtr<FStreamableHandle> UAssetManager::PreloadPrimaryAssets(const TArray<FPrimaryAssetId>& AssetsToLoad, const TArray<FName>& LoadBundles, bool bLoadRecursive, FAssetManagerLoadParams&& LoadParams, UE::FSourceLocation Location)
{
	TStringBuilder<256> DebugValid;
	TStringBuilder<256> DebugInvalid;

	TArray<FSoftObjectPath> PathsToLoad = UE::AssetManager::Private::GetPreloadPrimaryAssetsLoadList(this, AssetsToLoad, LoadBundles, bLoadRecursive, DebugValid, DebugInvalid);

	TSharedPtr<FStreamableHandle> ReturnHandle = LoadAssetList(MoveTemp(PathsToLoad), MoveTemp(LoadParams), FString(*DebugValid), MoveTemp(Location));

	ensureMsgf(ReturnHandle.IsValid(), TEXT("Requested preload of Primary Assets failed: %s"), *DebugInvalid);

	return ReturnHandle;
}

TSharedPtr<FStreamableHandle> UAssetManager::PredownloadPrimaryAssets(const TArray<FPrimaryAssetId>& AssetsToLoad, const TArray<FName>& LoadBundles, bool bLoadRecursive, FAssetManagerLoadParams&& LoadParams, UE::FSourceLocation Location)
{
	TStringBuilder<256> DebugValid;
	TStringBuilder<256> DebugInvalid;

	TArray<FSoftObjectPath> PathsToLoad = UE::AssetManager::Private::GetPreloadPrimaryAssetsLoadList(this, AssetsToLoad, LoadBundles, bLoadRecursive, DebugValid, DebugInvalid);

	TSharedPtr<FStreamableHandle> ReturnHandle = DownloadAssetList(MoveTemp(PathsToLoad), MoveTemp(LoadParams), FString(*DebugValid), MoveTemp(Location));

	ensureMsgf(ReturnHandle.IsValid(), TEXT("Requested predownload of Primary Assets failed: %s"), *DebugInvalid);

	return TSharedPtr<FStreamableHandle>();
}

void UAssetManager::OnAssetStateChangeCompleted(TSharedPtr<FStreamableHandle> BoundHandle, FPrimaryAssetId PrimaryAssetId)
{
	FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);

	if (NameData)
	{
		if (!ensureAlwaysMsgf(BoundHandle.IsValid(), TEXT("Received null handle for asset load request (%s)!"), *PrimaryAssetId.ToString()))
		{
			// That should never happen, but if it does - reset erroneous state
			NameData->PendingState.Reset(true);
			return;
		}

		if (!NameData->PendingState.GetHandle().IsValid() || NameData->PendingState.GetHandle() == BoundHandle)
		{
			// If an error occured, don't update the current state with a bad state
			if (BoundHandle->HasError())
			{
				const ::StreamableManager::FAdditionalContext* AdditionalContext = BoundHandle->GetError()->GetErrorContext<::StreamableManager::FAdditionalContext>();

				if (AdditionalContext)
				{
					UE_LOGF(LogAssetManager, Warning, "OnAssetStateChangeCompleted: Received error (%ls) loading asset! Primary asset id:(%ls) erroring asset:%ls", *BoundHandle->GetError()->GetErrorMessage(true).ToString(), *PrimaryAssetId.ToString(), *AdditionalContext->MissingObject);
				}
				else
				{
					UE_LOGF(LogAssetManager, Warning, "OnAssetStateChangeCompleted: Received error (%ls) loading asset! Primary asset id:(%ls)", *BoundHandle->GetError()->GetErrorMessage(true).ToString(), *PrimaryAssetId.ToString());
				}
				//UE_LOGFMT(LogAssetManager, Warning, "OnAssetStateChangeCompleted: Received error {Error} loading asset! Primary asset id:({PrimaryAssetId})", ("Error", BoundHandle->GetError()), ("PrimaryAssetId", *PrimaryAssetId.ToString()));
			}
			else if (BoundHandle->WasCanceled())
			{
				UE_LOGF(LogAssetManager, Verbose, "OnAssetStateChangeCompleted: Request was canceled (%ls)", *PrimaryAssetId.ToString());
			}
			else
			{
				NameData->CurrentState.MoveStateFrom(BoundHandle, MoveTemp(NameData->PendingState));
				
				NotifyOnAssetStateChangeCompleted(PrimaryAssetId);
			}

			NameData->PendingState.Reset(false);
		}
		else if (NameData->PendingState.GetHandle().IsValid())
		{
			UE_LOGF(LogAssetManager, Verbose, "OnAssetStateChangeCompleted: Received after pending data changed, ignoring (%ls)", *PrimaryAssetId.ToString());
		}
	}
	else
	{
		UE_LOGF(LogAssetManager, Error, "OnAssetStateChangeCompleted: Received for invalid asset! (%ls)", *PrimaryAssetId.ToString());
	}
}

TSharedPtr<FStreamableHandle> UAssetManager::LoadPrimaryAssets(const TArray<FPrimaryAssetId>& AssetsToLoad, const TArray<FName>& LoadBundles, FAssetManagerLoadParams&& LoadParams, UE::FSourceLocation Location)
{
	return ChangeBundleStateForPrimaryAssets(AssetsToLoad, LoadBundles, TArray<FName>(), true, MoveTemp(LoadParams), MoveTemp(Location));
}

TSharedPtr<FStreamableHandle> UAssetManager::LoadPrimaryAsset(const FPrimaryAssetId& AssetToLoad, const TArray<FName>& LoadBundles, FAssetManagerLoadParams&& LoadParams, UE::FSourceLocation Location)
{
	return LoadPrimaryAssets(TArray<FPrimaryAssetId>{AssetToLoad}, LoadBundles, MoveTemp(LoadParams), MoveTemp(Location));
}

TSharedPtr<FStreamableHandle> UAssetManager::LoadPrimaryAssetsWithType(FPrimaryAssetType PrimaryAssetType, const TArray<FName>& LoadBundles, FAssetManagerLoadParams&& LoadParams, UE::FSourceLocation Location)
{
	TArray<FPrimaryAssetId> Assets;
	GetPrimaryAssetIdList(PrimaryAssetType, Assets);
	return LoadPrimaryAssets(Assets, LoadBundles, MoveTemp(LoadParams), MoveTemp(Location));
}

TSharedPtr<FStreamableHandle> UAssetManager::GetPrimaryAssetHandle(const FPrimaryAssetId& PrimaryAssetId, bool bForceCurrent, TArray<FName>* Bundles) const
{
	const FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);

	if (!NameData)
	{
		return nullptr;
	}

	const FPrimaryAssetLoadState& LoadState = (bForceCurrent || !NameData->PendingState.IsValid()) ? NameData->CurrentState : NameData->PendingState;

	if (Bundles)
	{
		*Bundles = LoadState.GetBundleNames();
	}
	return LoadState.GetHandle();
}

bool UAssetManager::GetPrimaryAssetsWithBundleState(TArray<FPrimaryAssetId>& PrimaryAssetList, const TArray<FPrimaryAssetType>& ValidTypes, const TArray<FName>& RequiredBundles, const TArray<FName>& ExcludedBundles, bool bForceCurrent) const
{
	bool bFoundAny = false;

	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		if (ValidTypes.Num() > 0 && !ValidTypes.Contains(FPrimaryAssetType(TypePair.Key)))
		{
			// Skip this type
			continue;
		}

		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		for (const TPair<FName, FPrimaryAssetData>& NamePair : TypeData.GetAssets())
		{
			const FPrimaryAssetData& NameData = NamePair.Value;

			const FPrimaryAssetLoadState& LoadState = (bForceCurrent || !NameData.PendingState.IsValid()) ? NameData.CurrentState : NameData.PendingState;

			if (!LoadState.IsValid())
			{
				// Only allow loaded assets
				continue;
			}

			bool bFailedTest = false;

			// Check bundle requirements
			for (const FName& RequiredName : RequiredBundles)
			{
				if (!LoadState.GetBundleNames().Contains(RequiredName))
				{
					bFailedTest = true;
					break;
				}
			}

			for (const FName& ExcludedName : ExcludedBundles)
			{
				if (LoadState.GetBundleNames().Contains(ExcludedName))
				{
					bFailedTest = true;
					break;
				}
			}

			if (!bFailedTest)
			{
				PrimaryAssetList.Add(FPrimaryAssetId(TypePair.Key, NamePair.Key));
				bFoundAny = true;
			}
		}
	}

	return bFoundAny;
}

void UAssetManager::GetPrimaryAssetBundleStateMap(TMap<FPrimaryAssetId, TArray<FName>>& BundleStateMap, bool bForceCurrent) const
{
	BundleStateMap.Reset();

	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		for (const TPair<FName, FPrimaryAssetData>& NamePair : TypeData.GetAssets())
		{
			const FPrimaryAssetData& NameData = NamePair.Value;

			const FPrimaryAssetLoadState& LoadState = (bForceCurrent || !NameData.PendingState.IsValid()) ? NameData.CurrentState : NameData.PendingState;

			if (!LoadState.IsValid())
			{
				continue;
			}

			FPrimaryAssetId AssetID(TypePair.Key, NamePair.Key);

			BundleStateMap.Add(AssetID, TArray<FName>(LoadState.GetBundleNames()));
		}
	}
}

int32 UAssetManager::UnloadPrimaryAssets(const TArray<FPrimaryAssetId>& AssetsToUnload)
{
	int32 NumUnloaded = 0;

	for (const FPrimaryAssetId& PrimaryAssetId : AssetsToUnload)
	{
		FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);

		if (NameData)
		{
			// Undo current and pending
			if (NameData->CurrentState.IsValid() || NameData->PendingState.IsValid())
			{
				NumUnloaded++;
				NameData->CurrentState.Reset(true);
				NameData->PendingState.Reset(true);
			}
		}
		else
		{
			WarnAboutInvalidPrimaryAsset(PrimaryAssetId, TEXT("UnloadPrimaryAssets failed to find NameData"));
		}
	}

	return NumUnloaded;
}

int32 UAssetManager::UnloadPrimaryAsset(const FPrimaryAssetId& AssetToUnload)
{
	return UnloadPrimaryAssets(TArray<FPrimaryAssetId>{AssetToUnload});
}

int32 UAssetManager::UnloadPrimaryAssetsWithType(FPrimaryAssetType PrimaryAssetType)
{
	TArray<FPrimaryAssetId> Assets;
	GetPrimaryAssetIdList(PrimaryAssetType, Assets);
	return UnloadPrimaryAssets(Assets);
}

namespace UE::AssetManager::Private
{
	static FStreamableAsyncLoadParams GetStreamableAsyncLoadParams(FAssetManagerLoadParams&& LoadParams)
	{
		FStreamableAsyncLoadParams Params;
		Params.OnComplete = MoveTemp(LoadParams.OnComplete);
		Params.OnCancel = MoveTemp(LoadParams.OnCancel);
		Params.OnUpdate = MoveTemp(LoadParams.OnUpdate);
		Params.Priority = LoadParams.Priority;

		if (LoadParams.DownloadParams)
		{
			Params.DownloadParams.Emplace();
			Params.DownloadParams->Priority = LoadParams.DownloadParams->Priority;
			Params.DownloadParams->bInstallSoftReferences = LoadParams.DownloadParams->bInstallSoftReferences;
		}

		return Params;
	}
}

TSharedPtr<FStreamableHandle> UAssetManager::LoadAssetList(TArray<FSoftObjectPath>&& AssetList, FAssetManagerLoadParams&& LoadParams, FString&& DebugName, UE::FSourceLocation&& Location)
{
	TArray<int32> MissingChunks, ErrorChunks;

	if (bShouldAcquireMissingChunksOnLoad)
	{
		FindMissingChunkList(AssetList, MissingChunks, ErrorChunks);

		if (ErrorChunks.Num() > 0)
		{
			// At least one chunk doesn't exist, fail
			UE_LOGF(LogAssetManager, Error, "Failure loading %ls, Required chunk %d does not exist!", *DebugName, ErrorChunks[0]);
			return nullptr;
		}
	}

	TSharedPtr<FStreamableHandle> NewHandle;

	// SynchronousLoad doesn't make sense if chunks are missing
	// SynchronousLoad doesn't make sense if we are going to download packages	
	if (!LoadParams.DownloadParams && bShouldUseSynchronousLoad && MissingChunks.Num() == 0)
	{
		NewHandle = StreamableManager.RequestSyncLoad(MoveTemp(AssetList), false, MoveTemp(DebugName), MoveTemp(Location));
		FStreamableHandle::ExecuteDelegate(MoveTemp(LoadParams.OnComplete), NewHandle);
	}
	else
	{
		FStreamableAsyncLoadParams Params = UE::AssetManager::Private::GetStreamableAsyncLoadParams(MoveTemp(LoadParams));
		Params.TargetsToStream = MoveTemp(AssetList);
		Params.bManageActiveHandle = false;
		Params.bStartStalled = MissingChunks.Num() > 0;

		NewHandle = StreamableManager.RequestAsyncLoad(MoveTemp(Params), MoveTemp(DebugName), MoveTemp(Location));

		if (MissingChunks.Num() > 0 && NewHandle.IsValid())
		{
			AcquireChunkList(MissingChunks, FAssetManagerAcquireResourceDelegate(), EChunkPriority::Immediate, NewHandle);
		}
	}

	return NewHandle;
}

TSharedPtr<FStreamableHandle> UAssetManager::DownloadAssetList(TArray<FSoftObjectPath>&& AssetList, FAssetManagerLoadParams&& LoadParams, FString&& DebugName, UE::FSourceLocation&& Location)
{
	if (!LoadParams.DownloadParams)
	{
		UE_LOGF(LogAssetManager, Error, "Failure downloading %ls, no download params!", *DebugName);
		return nullptr;
	}

	FStreamableAsyncLoadParams Params = UE::AssetManager::Private::GetStreamableAsyncLoadParams(MoveTemp(LoadParams));
	Params.TargetsToStream = MoveTemp(AssetList);
	Params.bManageActiveHandle = false;
	Params.DownloadParams->bDownloadOnly = true;

	return StreamableManager.RequestAsyncLoad(MoveTemp(Params), MoveTemp(DebugName), MoveTemp(Location));
}

FAssetBundleEntry UAssetManager::GetAssetBundleEntry(const FPrimaryAssetId& BundleScope, FName BundleName) const
{
	if (const TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>* FoundMap = CachedAssetBundles.Find(BundleScope))
	{
		for (FAssetBundleEntry& Entry : (**FoundMap).Bundles)
		{
			if (Entry.BundleName == BundleName)
			{
				return Entry;
			}
		}
	}
	
	return FAssetBundleEntry();
}

bool UAssetManager::GetAssetBundleEntries(const FPrimaryAssetId& BundleScope, TArray<FAssetBundleEntry>& OutEntries) const
{
	if (const TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>* FoundMap = CachedAssetBundles.Find(BundleScope))
	{
		OutEntries.Append((**FoundMap).Bundles);
		return (**FoundMap).Bundles.Num() > 0;
	}
	
	return false;
}

bool UAssetManager::FindMissingChunkList(const TArray<FSoftObjectPath>& AssetList, TArray<int32>& OutMissingChunkList, TArray<int32>& OutErrorChunkList) const
{
	if (!bIsLoadingFromPakFiles)
	{
		return false;
	}

	// Cache of locations for chunk IDs
	TMap<int32, EChunkLocation::Type> ChunkLocationCache;

	// Grab chunk install
#if ENABLE_PLATFORM_CHUNK_INSTALL
	IPlatformChunkInstall* ChunkInstall = FPlatformMisc::GetPlatformChunkInstall();
#endif

	// Grab pak platform file
	FPakPlatformFile* Pak = (FPakPlatformFile*)FPlatformFileManager::Get().FindPlatformFile(TEXT("PakFile"));
	check(Pak);

	for (const FSoftObjectPath& Asset : AssetList)
	{
		FAssetData FoundData;
		GetAssetDataForPath(Asset, FoundData);
		TSet<int32> FoundChunks, MissingChunks, ErrorChunks;

		for (int32 PakchunkId : FoundData.GetChunkIDs())
		{
			if (!ChunkLocationCache.Contains(PakchunkId))
			{
#if ENABLE_PLATFORM_CHUNK_INSTALL
				EChunkLocation::Type Location = ChunkInstall->GetPakchunkLocation(PakchunkId);
#else
				EChunkLocation::Type Location = EChunkLocation::LocalFast;
#endif

				// If chunk install thinks the chunk is available, we need to double check with the pak system that it isn't
				// pending decryption
				if (Location >= EChunkLocation::LocalSlow && Pak->AnyChunksAvailable())
				{
					Location = Pak->GetPakChunkLocation(PakchunkId);
				}

				ChunkLocationCache.Add(PakchunkId, Location);
			}
			EChunkLocation::Type ChunkLocation = ChunkLocationCache[PakchunkId];

			switch (ChunkLocation)
			{			
			case EChunkLocation::DoesNotExist:
				ErrorChunks.Add(PakchunkId);
				break;
			case EChunkLocation::NotAvailable:
				MissingChunks.Add(PakchunkId);
				break;
			case EChunkLocation::LocalSlow:
			case EChunkLocation::LocalFast:
				FoundChunks.Add(PakchunkId);
				break;
			}
		}

		// Assets may be redundantly in multiple chunks, if we have any of the chunks then we have the asset
		if (FoundChunks.Num() == 0)
		{
			if (MissingChunks.Num() > 0)
			{
				int32 MissingChunkToAdd = -1;

				for (int32 MissingChunkId : MissingChunks)
				{
					if (OutMissingChunkList.Contains(MissingChunkId))
					{
						// This chunk is already scheduled, don't add a new one
						MissingChunkToAdd = -1;
						break;
					}
					else if (MissingChunkToAdd == -1)
					{
						// Add the first mentioned missing chunk
						MissingChunkToAdd = MissingChunkId;
					}
				}

				if (MissingChunkToAdd != -1)
				{
					OutMissingChunkList.Add(MissingChunkToAdd);
				}
			}
			else if (ErrorChunks.Num() > 0)
			{
				// Only have error chunks, report the errors
				for (int32 ErrorChunkId : ErrorChunks)
				{
					OutErrorChunkList.Add(ErrorChunkId);
				}
			}
		}
	}

	return OutMissingChunkList.Num() > 0 || OutErrorChunkList.Num() > 0;
}

void UAssetManager::AcquireChunkList(const TArray<int32>& ChunkList, FAssetManagerAcquireResourceDelegate CompleteDelegate, EChunkPriority::Type Priority, TSharedPtr<FStreamableHandle> StalledHandle)
{
#if ENABLE_PLATFORM_CHUNK_INSTALL
	FPendingChunkInstall* PendingChunkInstall = new(PendingChunkInstalls) FPendingChunkInstall;
	PendingChunkInstall->ManualCallback = MoveTemp(CompleteDelegate);
	PendingChunkInstall->RequestedChunks = ChunkList;
	PendingChunkInstall->PendingChunks = ChunkList;
	PendingChunkInstall->StalledStreamableHandle = StalledHandle;

	IPlatformChunkInstall* ChunkInstall = FPlatformMisc::GetPlatformChunkInstall();

	if (!ChunkInstallDelegateHandle.IsValid())
	{
		ChunkInstallDelegateHandle = ChunkInstall->AddChunkInstallDelegate(FPlatformChunkInstallDelegate::CreateUObject(this, &ThisClass::OnChunkDownloaded));
	}

	for (int32 MissingChunk : PendingChunkInstall->PendingChunks)
	{
		ChunkInstall->PrioritizePakchunk(MissingChunk, Priority);
	}
#endif
}

void UAssetManager::AcquireResourcesForAssetList(const TArray<FSoftObjectPath>& AssetList, FAssetManagerAcquireResourceDelegate CompleteDelegate, EChunkPriority::Type Priority)
{
	AcquireResourcesForAssetList(AssetList, FAssetManagerAcquireResourceDelegateEx::CreateLambda([CompleteDelegate = MoveTemp(CompleteDelegate)](bool bSuccess, const TArray<int32>& Unused) { CompleteDelegate.ExecuteIfBound(bSuccess); }), Priority);
}

void UAssetManager::AcquireResourcesForAssetList(const TArray<FSoftObjectPath>& AssetList, FAssetManagerAcquireResourceDelegateEx CompleteDelegate, EChunkPriority::Type Priority)
{
	TArray<int32> MissingChunks, ErrorChunks;
	FindMissingChunkList(AssetList, MissingChunks, ErrorChunks);
	if (ErrorChunks.Num() > 0)
	{
		// At least one chunk doesn't exist, fail
		FStreamableDelegateWithHandle TempDelegate = FStreamableDelegateWithHandle::CreateLambda(
			[CompleteDelegate = MoveTemp(CompleteDelegate), MissingChunks](const TSharedPtr<struct FStreamableHandle>&) { CompleteDelegate.ExecuteIfBound(false, MissingChunks); });
		FStreamableHandle::ExecuteDelegate(MoveTemp(TempDelegate));
	}
	else if (MissingChunks.Num() == 0)
	{
		// All here, schedule the callback
		FStreamableDelegateWithHandle TempDelegate = FStreamableDelegateWithHandle::CreateLambda(
			[CompleteDelegate = MoveTemp(CompleteDelegate)](const TSharedPtr<struct FStreamableHandle>&) { CompleteDelegate.ExecuteIfBound(true, TArray<int32>()); });
		FStreamableHandle::ExecuteDelegate(MoveTemp(TempDelegate));
	}
	else
	{
		AcquireChunkList(MissingChunks, FAssetManagerAcquireResourceDelegate::CreateLambda([CompleteDelegate = MoveTemp(CompleteDelegate), MissingChunks](bool bSuccess) { CompleteDelegate.ExecuteIfBound(bSuccess, MissingChunks); }), Priority, nullptr);
	}
}

void UAssetManager::AcquireResourcesForPrimaryAssetList(const TArray<FPrimaryAssetId>& PrimaryAssetList, FAssetManagerAcquireResourceDelegate CompleteDelegate, EChunkPriority::Type Priority)
{
	TSet<FSoftObjectPath> PathsToLoad;
	TSharedPtr<FStreamableHandle> ReturnHandle;

	for (const FPrimaryAssetId& PrimaryAssetId : PrimaryAssetList)
	{
		FPrimaryAssetData* NameData = GetNameData(PrimaryAssetId);

		if (NameData)
		{
			// Gather asset refs
			const FSoftObjectPath& AssetPath = NameData->GetAssetPtr().ToSoftObjectPath();

			if (!AssetPath.IsNull())
			{
				// Dynamic types can have no base asset path
				PathsToLoad.Add(AssetPath);
			}

			TArray<FAssetBundleEntry> BundleEntries;
			GetAssetBundleEntries(PrimaryAssetId, BundleEntries);
			for (const FAssetBundleEntry& Entry : BundleEntries)
			{
				if (Entry.IsValid())
				{
					for (const FTopLevelAssetPath& Path : Entry.AssetPaths)
					{
						PathsToLoad.Emplace(FSoftObjectPath(Path));
					}
				}
			}
		}
		else
		{
			WarnAboutInvalidPrimaryAsset(PrimaryAssetId, TEXT("AcquireResourcesForPrimaryAssetList failed to find NameData"));
		}
	}

	AcquireResourcesForAssetList(PathsToLoad.Array(), MoveTemp(CompleteDelegate), Priority);
}

bool UAssetManager::GetResourceAcquireProgress(int32& OutAcquiredCount, int32& OutRequestedCount) const
{
	OutAcquiredCount = OutRequestedCount = 0;
	// Iterate pending callbacks, in order they were added
	for (const FPendingChunkInstall& PendingChunkInstall : PendingChunkInstalls)
	{
		OutRequestedCount += PendingChunkInstall.RequestedChunks.Num();
		OutAcquiredCount += (PendingChunkInstall.RequestedChunks.Num() - PendingChunkInstall.PendingChunks.Num());
	}

	return PendingChunkInstalls.Num() > 0;
}

void UAssetManager::OnChunkDownloaded(uint32 ChunkId, bool bSuccess)
{
#if ENABLE_PLATFORM_CHUNK_INSTALL
	IPlatformChunkInstall* ChunkInstall = FPlatformMisc::GetPlatformChunkInstall();

	// Iterate pending callbacks, in order they were added
	for (int32 i = 0; i < PendingChunkInstalls.Num(); i++)
	{
		// Make a copy so if we resize the array it's safe
		FPendingChunkInstall PendingChunkInstall = PendingChunkInstalls[i];
		if (PendingChunkInstall.PendingChunks.Contains(ChunkId))
		{
			bool bFailed = !bSuccess;
			TArray<int32> NewPendingList;
			
			// Check all chunks if they are done or failed
			for (int32 PendingPakchunkId : PendingChunkInstall.PendingChunks)
			{
				EChunkLocation::Type ChunkLocation = ChunkInstall->GetPakchunkLocation(PendingPakchunkId);

				switch (ChunkLocation)
				{
				case EChunkLocation::DoesNotExist:
					bFailed = true;
					break;
				case EChunkLocation::NotAvailable:
					NewPendingList.Add(PendingPakchunkId);
					break;
				}
			}

			if (bFailed)
			{
				// Resize array first
				PendingChunkInstalls.RemoveAt(i);
				i--;

				if (PendingChunkInstall.StalledStreamableHandle.IsValid())
				{
					PendingChunkInstall.StalledStreamableHandle->CancelHandle();
				}

				PendingChunkInstall.ManualCallback.ExecuteIfBound(false);
			}
			else if (NewPendingList.Num() == 0)
			{
				// Resize array first
				PendingChunkInstalls.RemoveAt(i);
				i--;

				if (PendingChunkInstall.StalledStreamableHandle.IsValid())
				{
					// Now that this stalled load can resume, we need to clear all of it's requested assets
					// from the known missing list, just in case we ever previously tried to load them from
					// before the chunk was installed/decrypted
					TArray<FSoftObjectPath> RequestedAssets;
					PendingChunkInstall.StalledStreamableHandle->GetRequestedAssets(RequestedAssets);
					for (const FSoftObjectPath& Path : RequestedAssets)
					{
						FName Name(*Path.GetLongPackageName());
						if (FLinkerLoad::IsKnownMissingPackage(Name))
						{
							FLinkerLoad::RemoveKnownMissingPackage(Name);
						}
					}
					PendingChunkInstall.StalledStreamableHandle->StartStalledHandle();
				}

				PendingChunkInstall.ManualCallback.ExecuteIfBound(true);
			}
			else
			{
				PendingChunkInstalls[i].PendingChunks = NewPendingList;
			}
		}
	}
#endif
}

bool UAssetManager::OnAssetRegistryAvailableAfterInitialization(FName InName, FAssetRegistryState& OutNewState)
{
#if WITH_EDITOR
	UE_LOGF(LogAssetManager, Warning, "UAssetManager::OnAssetRegistryAvailableAfterInitialization is only supported in cooked builds, but was called from the editor!");
	return false;
#else

	bool bLoaded = false;
	double RegistrationTime = 0.0;

	{
		SCOPE_SECONDS_COUNTER(RegistrationTime);

		IAssetRegistry& LocalAssetRegistry = GetAssetRegistry();
		
		{
			TArray<uint8> Bytes;
			const FString Filename = FPaths::ProjectDir() / (TEXT("AssetRegistry") + InName.ToString()) + TEXT(".bin");
			// Optimization: if we're using pak files then only search paks (avoid unnecessary fallback to loose)
			FPakPlatformFile* PakPlatformFile = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(FPakPlatformFile::GetTypeName()));
			bool bFileExists = PakPlatformFile != nullptr ? PakPlatformFile->FindFileInPakFiles(*Filename) : FPaths::FileExists(Filename);
			if (bFileExists && FFileHelper::LoadFileToArray(Bytes, *Filename))
			{
				bLoaded = true;
				FMemoryReader Ar(Bytes);
				OutNewState.Load(Ar);
			}
		}

		if (bLoaded)
		{
			LocalAssetRegistry.AppendState(OutNewState);

			// Invalidate the package localization cache for any updated mount roots
			{
				TArray<FString> RootPathsToInvalidate;
				OutNewState.EnumerateAllPaths([&RootPathsToInvalidate](const FName PackagePath)
				{
					FString RootPath = FPackageName::SplitPackageNameRoot(PackagePath, nullptr);
					RootPath.InsertAt(0, TEXT('/'));
					RootPathsToInvalidate.AddUnique(MoveTemp(RootPath));
				});
				for (const FString& RootPathToInvalidate : RootPathsToInvalidate)
				{
					FPackageLocalizationManager::Get().InvalidateRootSourcePath(RootPathToInvalidate);
				}
			}

			TArray<FAssetData> NewAssetData;
			bool bRebuildReferenceList = false;
			if (OutNewState.GetAllAssets(TSet<FName>(), NewAssetData))
			{
				// Temporary performance fix while waiting for Ben Zeigler's AssetManager registry scanning changes
				const bool bPathExclusionsExists = GetSettings().DirectoriesToExclude.Num() > 0;

				for (const FAssetData& AssetData : NewAssetData)
				{
					if (!bPathExclusionsExists || !IsPathExcludedFromScan(AssetData.PackageName.ToString()))
					{
						FPrimaryAssetId PrimaryAssetId = AssetData.GetPrimaryAssetId();
						if (PrimaryAssetId.IsValid() && AssetData.IsTopLevelAsset())
						{
							TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetId.PrimaryAssetType);
							if (FoundType)
							{
								FPrimaryAssetTypeData& TypeData = FoundType->Get();
								if (ShouldScanPrimaryAssetType(TypeData.Info))
								{
									// Make sure it's in a valid path
									bool bFoundPath = false;
									for (const FString& Path : TypeData.RealAssetScanPaths)
									{
										if (AssetData.PackageName.ToString().StartsWith(Path))
										{
											bFoundPath = true;
											break;
										}
									}

									if (bFoundPath)
									{
										FString GuidString;
										if (AssetData.GetTagValue(GetEncryptionKeyAssetTagName(), GuidString))
										{
											FGuid Guid;
											FGuid::Parse(GuidString, Guid);
											check(!PrimaryAssetEncryptionKeyCache.Contains(PrimaryAssetId));
											PrimaryAssetEncryptionKeyCache.Add(PrimaryAssetId, Guid);
											UE_LOGF(LogAssetManager, Verbose, "Found encrypted primary asset '%ls' using keys '%ls'", *PrimaryAssetId.PrimaryAssetName.ToString(), *GuidString);
										}

										// Check exclusion path
										if (TryUpdateCachedAssetData(PrimaryAssetId, AssetData, false))
										{
											bRebuildReferenceList = true;
										}
									}
								}
							}
						}
					}
				}
			}

			if (bRebuildReferenceList)
			{
				OnObjectReferenceListInvalidated();
			}
		}
	}

	UE_CLOGF(bLoaded, LogAssetManager, Log, "Registered new asset registry '%ls' in %.4fs", *InName.ToString(), RegistrationTime);
	return bLoaded;
#endif
}

FPrimaryAssetData* UAssetManager::GetNameData(const FPrimaryAssetId& PrimaryAssetId, bool bCheckRedirector)
{
	return const_cast<FPrimaryAssetData*>(AsConst(*this).GetNameData(PrimaryAssetId));
}

const FPrimaryAssetData* UAssetManager::GetNameData(const FPrimaryAssetId& PrimaryAssetId, bool bCheckRedirector) const
{
	const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetId.PrimaryAssetType);

	if (FoundType)
	{
		const FPrimaryAssetData* FoundName = (*FoundType)->GetAssets().Find(PrimaryAssetId.PrimaryAssetName);
		
		if (FoundName)
		{
			return FoundName;
		}
	}

	// Try redirected name
	if (bCheckRedirector)
	{
		FPrimaryAssetId RedirectedId = GetRedirectedPrimaryAssetId(PrimaryAssetId);

		if (RedirectedId.IsValid())
		{
			// Recursively call self, but turn off recursion flag
			return GetNameData(RedirectedId, false);
		}
	}

	return nullptr;
}

void UAssetManager::OnObjectReferenceListInvalidated()
{
	bIsManagementDatabaseCurrent = false;
	bObjectReferenceListDirty = true;
}

void UAssetManager::CallPreGarbageCollect()
{
	PreGarbageCollect();
}

void UAssetManager::PreGarbageCollect()
{
	if (bObjectReferenceListDirty)
	{
		bObjectReferenceListDirty = false;
		ObjectReferenceList.Reset();

		// Iterate primary asset map
		for (TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
		{
			FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

			// Add base class in case it's a blueprint
			if (!TypeData.Info.bIsDynamicAsset)
			{
				ObjectReferenceList.AddUnique(TypeData.Info.AssetBaseClassLoaded);
			}
		}
	}
}

void UAssetManager::LoadRedirectorMaps()
{
	AssetPathRedirects.Reset();
	PrimaryAssetIdRedirects.Reset();
	PrimaryAssetTypeRedirects.Reset();

	const UAssetManagerSettings& Settings = GetSettings();

	for (const FAssetManagerRedirect& Redirect : Settings.PrimaryAssetTypeRedirects)
	{
		PrimaryAssetTypeRedirects.Add(FName(*Redirect.Old), FName(*Redirect.New));
	}

	for (const FAssetManagerRedirect& Redirect : Settings.PrimaryAssetIdRedirects)
	{
		PrimaryAssetIdRedirects.Add(Redirect.Old, Redirect.New);
	}

	for (const FAssetManagerRedirect& Redirect : Settings.AssetPathRedirects)
	{
		AssetPathRedirects.Add(FSoftObjectPath(*Redirect.Old), FSoftObjectPath(*Redirect.New));
	}

	// Collapse all redirects to resolve recursive relationships.
	for (const TPair<FSoftObjectPath, FSoftObjectPath>& Pair : AssetPathRedirects)
	{
		const FSoftObjectPath OldPath = Pair.Key;
		FSoftObjectPath NewPath = Pair.Value;
		TSet<FSoftObjectPath> CollapsedPaths;
		CollapsedPaths.Add(OldPath);
		CollapsedPaths.Add(NewPath);
		while (FSoftObjectPath* NewPathValue = AssetPathRedirects.Find(NewPath)) // Does the NewPath exist as a key?
		{
			NewPath = *NewPathValue;
			if (CollapsedPaths.Contains(NewPath))
			{
				UE_LOGF(LogAssetManager, Error, "AssetPathRedirect cycle detected when redirecting: %ls to %ls", *OldPath.ToString(), *NewPath.ToString());
				break;
			}
			else
			{
				CollapsedPaths.Add(NewPath);
			}
		}
		AssetPathRedirects[OldPath] = NewPath;
	}
}

FPrimaryAssetId UAssetManager::GetRedirectedPrimaryAssetId(const FPrimaryAssetId& OldId) const
{
	FString OldIdString = OldId.ToString();

	const FString* FoundId = PrimaryAssetIdRedirects.Find(OldIdString);

	if (FoundId)
	{
		return FPrimaryAssetId(*FoundId);
	}

	// Now look for type redirect
	const FName* FoundType = PrimaryAssetTypeRedirects.Find(OldId.PrimaryAssetType);

	if (FoundType)
	{
		return FPrimaryAssetId(*FoundType, OldId.PrimaryAssetName);
	}

	return FPrimaryAssetId();
}

void UAssetManager::GetPreviousPrimaryAssetIds(const FPrimaryAssetId& NewId, TArray<FPrimaryAssetId>& OutOldIds) const
{
	FString NewIdString = NewId.ToString();

	for (const TPair<FString, FString>& Redirect : PrimaryAssetIdRedirects)
	{
		if (Redirect.Value == NewIdString)
		{
			OutOldIds.AddUnique(FPrimaryAssetId(Redirect.Key));
		}
	}

	// Also look for type redirects
	for (const TPair<FName, FName>& Redirect : PrimaryAssetTypeRedirects)
	{
		if (Redirect.Value == NewId.PrimaryAssetType.GetName())
		{
			OutOldIds.AddUnique(FPrimaryAssetId(Redirect.Key, NewId.PrimaryAssetName));
		}
	}
}

FSoftObjectPath UAssetManager::GetRedirectedAssetPath(const FSoftObjectPath& ObjectPath) const
{
	const FSoftObjectPath* RedirectedName = AssetPathRedirects.Find(ObjectPath.GetWithoutSubPath());
	if (!RedirectedName)
	{
		return FSoftObjectPath();
	}
	return FSoftObjectPath(RedirectedName->GetAssetPath(), ObjectPath.GetSubPathString());
}

void UAssetManager::ExtractSoftObjectPaths(const UStruct* Struct, const void* StructValue, TArray<FSoftObjectPath>& FoundAssetReferences, const TArray<FName>& PropertiesToSkip) const
{
	if (!ensure(Struct && StructValue))
	{
		return;
	}

	for (TPropertyValueIterator<const FProperty> It(Struct, StructValue); It; ++It)
	{
		const FProperty* Property = It.Key();
		const void* PropertyValue = It.Value();
		
		if (PropertiesToSkip.Contains(Property->GetFName()))
		{
			It.SkipRecursiveProperty();
			continue;
		}

		FSoftObjectPath FoundRef;
		if (const FSoftClassProperty* AssetClassProp = CastField<FSoftClassProperty>(Property))
		{
			const TSoftClassPtr<UObject>* AssetClassPtr = reinterpret_cast<const TSoftClassPtr<UObject>*>(PropertyValue);
			if (AssetClassPtr)
			{
				FoundRef = AssetClassPtr->ToSoftObjectPath();
			}
		}
		else if (const FSoftObjectProperty* AssetProp = CastField<FSoftObjectProperty>(Property))
		{
			const TSoftObjectPtr<UObject>* AssetPtr = reinterpret_cast<const TSoftObjectPtr<UObject>*>(PropertyValue);
			if (AssetPtr)
			{
				FoundRef = AssetPtr->ToSoftObjectPath();
			}
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			// SoftClassPath is binary identical with SoftObjectPath
			if (StructProperty->Struct == TBaseStructure<FSoftObjectPath>::Get() || StructProperty->Struct == TBaseStructure<FSoftClassPath>::Get())
			{
				const FSoftObjectPath* AssetRefPtr = reinterpret_cast<const FSoftObjectPath*>(PropertyValue);
				if (AssetRefPtr)
				{
					FoundRef = *AssetRefPtr;
				}

				// Skip recursion, we don't care about the raw string property
				It.SkipRecursiveProperty();
			}
		}
		if (!FoundRef.IsNull())
		{
			FoundAssetReferences.AddUnique(FoundRef);
		}
	}
}

bool UAssetManager::GetAssetDataForPath(const FSoftObjectPath& ObjectPath, FAssetData& AssetData) const
{
	if (ObjectPath.IsNull())
	{
		return false;
	}

	IAssetRegistry& AssetRegistry = GetAssetRegistry();

	FString AssetPath = ObjectPath.ToString();

	// First check local redirector
	FSoftObjectPath RedirectedPath = GetRedirectedAssetPath(ObjectPath);

	if (RedirectedPath.IsValid())
	{
		AssetPath = RedirectedPath.ToString();
	}

	GetAssetDataForPathInternal(AssetRegistry, AssetPath, AssetData);

#if WITH_EDITOR
	// Cooked data has the asset data already set up. Uncooked builds may need to manually scan for this file
	if (!AssetData.IsValid())
	{
		ScanPathsSynchronous(TArray<FString>{AssetPath});

		GetAssetDataForPathInternal(AssetRegistry, AssetPath, AssetData);
	}

	// Handle redirector chains
	FAssetDataTagMapSharedView::FFindTagResult Result = AssetData.TagsAndValues.FindTag("DestinationObject");
	while (Result.IsSet())
	{
		FString DestinationObjectPath = Result.GetValue();
		ConstructorHelpers::StripObjectClass(DestinationObjectPath);
		AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(DestinationObjectPath));
		Result = AssetData.TagsAndValues.FindTag("DestinationObject");
	}

#endif

	return AssetData.IsValid();
}

static bool EndsWithBlueprint(const FTopLevelAssetPath& Name)
{
	// Numbered names can't end with Blueprint
	TStringBuilder<512> AssetName;
	Name.GetAssetName().ToString(AssetName);
	return AssetName.ToView().EndsWith(TEXT("Blueprint"));
}

static bool ContainsSubobjectDelimiter(FName Name)
{
	TCHAR Buffer[NAME_SIZE];
	FStringView View(Buffer, Name.GetPlainNameString(Buffer));
	return UE::String::FindFirstChar(View, SUBOBJECT_DELIMITER_CHAR) != INDEX_NONE;
}

template<EAssetDataCanBeSubobject ScanForSubobject>
bool TryToSoftObjectPath(const FAssetData& AssetData, FSoftObjectPath& OutObjectPath)
{
	if (!AssetData.IsValid())
	{
		OutObjectPath = FSoftObjectPath();
		return false;
	}
	else if (EndsWithBlueprint(AssetData.AssetClassPath))
	{
		TStringBuilder<256> AssetPath;
		AssetPath << AssetData.ToSoftObjectPath() << TEXT("_C");
		OutObjectPath = FSoftObjectPath(FStringView(AssetPath));
		return true;
	}
	else if (ScanForSubobject == EAssetDataCanBeSubobject::Yes)
	{
		OutObjectPath = FSoftObjectPath(AssetData.GetSoftObjectPath());
		return true;
	}
	else
	{
		if (!AssetData.IsTopLevelAsset())
		{
			OutObjectPath = FSoftObjectPath();
			return false;
		}
		OutObjectPath = AssetData.GetSoftObjectPath();
		return true;
	}
}

FSoftObjectPath UAssetManager::GetAssetPathForData(const FAssetData& AssetData) const
{
	FSoftObjectPath Result;
	TryToSoftObjectPath<EAssetDataCanBeSubobject::Yes>(AssetData, Result);
	return Result;
}

void UAssetManager::GetPakStringToChunkIdMapping(TMap<FString, int32>& StringOverrideToChunkId)
{
	TMap<int32, FString> ChunkIdStringOverride;
	GetPakChunkIdToStringMapping(ChunkIdStringOverride);

	for (const TPair<int32, FString>& IdAndStringPair : ChunkIdStringOverride)
	{
		StringOverrideToChunkId.Add(IdAndStringPair.Value, IdAndStringPair.Key);
	}
}

void UAssetManager::GetAssetDataForPathInternal(IAssetRegistry& AssetRegistry, const FString& AssetPath, OUT FAssetData& OutAssetData) const
{
	// We're a class if our path is foo.foo_C
	bool bIsClass = AssetPath.EndsWith(TEXT("_C"), ESearchCase::CaseSensitive) && !AssetPath.Contains(TEXT("_C."), ESearchCase::CaseSensitive);

	// If we're a class, first look for the asset data without the trailing _C
	// We do this first because in cooked builds you have to search the asset registry for the Blueprint, not the class itself
	if (bIsClass)
	{
		// We need to strip the class suffix because the asset registry has it listed by blueprint name
		
		OutAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(AssetPath.LeftChop(2)), bIncludeOnlyOnDiskAssets);

		if (OutAssetData.IsValid())
		{
			return;
		}
	}

	OutAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(AssetPath), bIncludeOnlyOnDiskAssets);
}

bool UAssetManager::WriteCustomReport(FString FileName, TArray<FString>& FileLines) const
{
	// Has a report been generated
	bool ReportGenerated = false;

	// Ensure we have a log to write
	if (FileLines.Num())
	{
		// Create the file name		
		FString FileLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() + TEXT("Reports/"));
		FString FullPath = FString::Printf(TEXT("%s%s"), *FileLocation, *FileName);

		// save file
		FArchive* LogFile = IFileManager::Get().CreateFileWriter(*FullPath);

		if (LogFile != NULL)
		{
			for (int32 Index = 0; Index < FileLines.Num(); ++Index)
			{
				FString LogEntry = FString::Printf(TEXT("%s"), *FileLines[Index]) + LINE_TERMINATOR;
				LogFile->Serialize(TCHAR_TO_ANSI(*LogEntry), LogEntry.Len());
			}

			LogFile->Close();
			delete LogFile;

			// A report has been generated
			ReportGenerated = true;
		}
	}

	return ReportGenerated;
}

static FAutoConsoleCommand CVarDumpAssetTypeSummary(
	TEXT("AssetManager.DumpTypeSummary"),
	TEXT("Shows a summary of types known about by the asset manager"),
	FConsoleCommandDelegate::CreateStatic(UAssetManager::DumpAssetTypeSummary),
	ECVF_Cheat);

void UAssetManager::DumpAssetTypeSummary()
{
	check(UAssetManager::IsInitialized());

	UAssetManager& Manager = Get();
	TArray<FPrimaryAssetTypeInfo> TypeInfos;

	Manager.GetPrimaryAssetTypeInfoList(TypeInfos);

	TypeInfos.Sort([](const FPrimaryAssetTypeInfo& LHS, const FPrimaryAssetTypeInfo& RHS) { return LHS.PrimaryAssetType.LexicalLess(RHS.PrimaryAssetType); });

	UE_LOGF(LogAssetManager, Log, "=========== Asset Manager Type Summary ===========");

	for (const FPrimaryAssetTypeInfo& TypeInfo : TypeInfos)
	{
		UE_LOGF(LogAssetManager, Log, "  %ls: Class %ls, Count %d, Paths %ls", *TypeInfo.PrimaryAssetType.ToString(), *TypeInfo.AssetBaseClassLoaded->GetName(), TypeInfo.NumberOfAssets, *FString::Join(TypeInfo.AssetScanPaths, TEXT(", ")));
	}
}

static FAutoConsoleCommand CVarDumpLoadedAssetState(
	TEXT("AssetManager.DumpLoadedAssets"),
	TEXT("Shows a list of all loaded primary assets and bundles"),
	FConsoleCommandDelegate::CreateStatic(UAssetManager::DumpLoadedAssetState),
	ECVF_Cheat);

void UAssetManager::DumpLoadedAssetState()
{
	check(UAssetManager::IsInitialized());

	UAssetManager& Manager = Get();
	TArray<FPrimaryAssetTypeInfo> TypeInfos;

	Manager.GetPrimaryAssetTypeInfoList(TypeInfos);

	TypeInfos.Sort([](const FPrimaryAssetTypeInfo& LHS, const FPrimaryAssetTypeInfo& RHS) { return LHS.PrimaryAssetType.LexicalLess(RHS.PrimaryAssetType); });

	UE_LOGF(LogAssetManager, Log, "=========== Asset Manager Loaded Asset State ===========");

	for (const FPrimaryAssetTypeInfo& TypeInfo : TypeInfos)
	{
		struct FLoadedInfo
		{
			FName AssetName;
			bool bPending;
			FString BundleState;

			FLoadedInfo(FName InAssetName, bool bInPending, const FString& InBundleState) : AssetName(InAssetName), bPending(bInPending), BundleState(InBundleState) {}
		};

		TArray<FLoadedInfo> LoadedInfos;

		FPrimaryAssetTypeData& TypeData = Manager.AssetTypeMap.Find(TypeInfo.PrimaryAssetType)->Get();

		for (const TPair<FName, FPrimaryAssetData>& NamePair : TypeData.GetAssets())
		{
			const FPrimaryAssetData& NameData = NamePair.Value;

			if (NameData.PendingState.IsValid() || NameData.CurrentState.IsValid())
			{
				const FPrimaryAssetLoadState& LoadState = (!NameData.PendingState.IsValid()) ? NameData.CurrentState : NameData.PendingState;

				FString BundleString;

				for (const FName& BundleName : LoadState.GetBundleNames())
				{
					if (!BundleString.IsEmpty())
					{
						BundleString += TEXT(", ");
					}
					BundleString += BundleName.ToString();
				}

				LoadedInfos.Emplace(NamePair.Key, NameData.PendingState.IsValid(), BundleString);
			}
		}

		if (LoadedInfos.Num() > 0)
		{
			UE_LOGF(LogAssetManager, Log, "  Type %ls:", *TypeInfo.PrimaryAssetType.ToString());

			LoadedInfos.Sort([](const FLoadedInfo& LHS, const FLoadedInfo& RHS) { return LHS.AssetName.LexicalLess(RHS.AssetName); });

			for (FLoadedInfo& LoadedInfo : LoadedInfos)
			{
				UE_LOGF(LogAssetManager, Log, "    %ls: %ls, (%ls)", *LoadedInfo.AssetName.ToString(), LoadedInfo.bPending ? TEXT("pending load") : TEXT("loaded"), *LoadedInfo.BundleState);
			}
		}	
	}
}

static FAutoConsoleCommand CVarDumpBundlesForAsset(
	TEXT("AssetManager.DumpBundlesForAsset"),
	TEXT("Shows a list of all bundles for the specified primary asset by primary asset id (i.e. Map:Entry)"),
	FConsoleCommandWithArgsDelegate::CreateStatic(UAssetManager::DumpBundlesForAsset),
	ECVF_Cheat);

void UAssetManager::DumpBundlesForAsset(const TArray<FString>& Args)
{
	if (Args.Num() < 1)
	{
		UE_LOGF(LogAssetManager, Warning, "Too few arguments for DumpBundlesForAsset. Include the primary asset id (i.e. Map:Entry)");
		return;
	}

	FString PrimaryAssetIdString = Args[0];
	if (!PrimaryAssetIdString.Contains(TEXT(":")))
	{
		UE_LOGF(LogAssetManager, Warning, "Incorrect argument for DumpBundlesForAsset. Arg should be the primary asset id (i.e. Map:Entry)");
		return;
	}

	check(UAssetManager::IsInitialized());

	UAssetManager& Manager = Get();

	FPrimaryAssetId PrimaryAssetId(PrimaryAssetIdString);
	const TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>* FoundMap = Manager.CachedAssetBundles.Find(PrimaryAssetId);
	if (!FoundMap)
	{
		UE_LOGF(LogAssetManager, Display, "Could not find bundles for primary asset %ls.", *PrimaryAssetIdString);
		return;
	}

	UE_LOGF(LogAssetManager, Display, "Dumping bundles for primary asset %ls...", *PrimaryAssetIdString);
	for (const FAssetBundleEntry& Entry : (**FoundMap).Bundles)
	{
		UE_LOGF(LogAssetManager, Display, "  Bundle: %ls (%d assets)", *Entry.BundleName.ToString(), Entry.AssetPaths.Num());
		for (const FTopLevelAssetPath& Path : Entry.AssetPaths)
		{
			UE_LOGF(LogAssetManager, Display, "    %ls", *Path.ToString());
		}
	}
}

static FAutoConsoleCommand CVarDumpAssetRegistryInfo(
	TEXT("AssetManager.DumpAssetRegistryInfo"),
	TEXT("Dumps extended info about asset registry to log"),
	FConsoleCommandDelegate::CreateStatic(UAssetManager::DumpAssetRegistryInfo),
	ECVF_Cheat);

void UAssetManager::DumpAssetRegistryInfo()
{
	UE_LOGF(LogAssetManager, Log, "=========== Asset Registry Summary ===========");
	UE_LOGF(LogAssetManager, Log, "Current Registry Memory:");

	UAssetManager& Manager = Get();

	// Output sizes
	Manager.GetAssetRegistry().GetAllocatedSize(true);

#if WITH_EDITOR
	UE_LOGF(LogAssetManager, Log, "Estimated Cooked Registry Memory:");

	FAssetRegistryState State;
	FAssetRegistrySerializationOptions SaveOptions;

	Manager.GetAssetRegistry().InitializeSerializationOptions(SaveOptions);
	Manager.GetAssetRegistry().InitializeTemporaryAssetRegistryState(State, SaveOptions);

	State.GetAllocatedSize(true);
#endif
}

static FAutoConsoleCommand CVarDumpReferencersForPackage(
	TEXT("AssetManager.DumpReferencersForPackage"),
	TEXT("Generates a graph viz and log file of all references to a specified package"),
	FConsoleCommandWithArgsDelegate::CreateStatic(UAssetManager::DumpReferencersForPackage),
	ECVF_Cheat);

void UAssetManager::DumpReferencersForPackage(const TArray< FString >& PackageNames)
{
	if (PackageNames.Num() == 0)
	{
		return;
	}

	check(UAssetManager::IsInitialized());
	UAssetManager& Manager = Get();
	IAssetRegistry& AssetRegistry = Manager.GetAssetRegistry();

	TArray<FString> ReportLines;

	ReportLines.Add(TEXT("digraph { "));

	for (const FString& PackageString : PackageNames)
	{
		TArray<FAssetIdentifier> FoundReferencers;

		AssetRegistry.GetReferencers(FName(*PackageString), FoundReferencers, UE::AssetRegistry::EDependencyCategory::Package);

		for (const FAssetIdentifier& Identifier : FoundReferencers)
		{
			FString ReferenceString = Identifier.ToString();

			ReportLines.Add(FString::Printf(TEXT("\t\"%s\" -> \"%s\";"), *ReferenceString, *PackageString));

			UE_LOGF(LogAssetManager, Log, "%ls depends on %ls", *ReferenceString, *PackageString);
		}
	}

	ReportLines.Add(TEXT("}"));

	Manager.WriteCustomReport(FString::Printf(TEXT("ReferencersForPackage%s%s.gv"), *PackageNames[0], *FDateTime::Now().ToString()), ReportLines);
}

void UAssetManager::GetAllReferencersForPackage(TSet<FAssetData>& OutFoundAssets, const TArray<FName>& InPackageNames, int32 MaxDepth)
{
	ensureMsgf(MaxDepth > 0, TEXT("Max depth to search for referencers should be greater than 0"));

	UAssetManager& Manager = Get();
	IAssetRegistry& AssetRegistry = Manager.GetAssetRegistry();

	TSet<FName> PackagesForNextLoop;
	TSet<FName> CurrentPackagesToProcess;

	CurrentPackagesToProcess.Append(InPackageNames);
	for (int32 CurrentDepth = 0; CurrentDepth < MaxDepth; CurrentDepth++)
	{
		for (const FName& PackageName : CurrentPackagesToProcess)
		{
			FARFilter Filter;
			Filter.bIncludeOnlyOnDiskAssets = true;
			AssetRegistry.GetReferencers(PackageName, Filter.PackageNames, UE::AssetRegistry::EDependencyCategory::Package);

			TArray<FAssetData> AssetReferencers;
			AssetRegistry.GetAssets(Filter, AssetReferencers);
			OutFoundAssets.Append(AssetReferencers);
			for (const FAssetData& AssetData : AssetReferencers)
			{
				PackagesForNextLoop.Add(AssetData.PackageName);
			}
		}
		CurrentPackagesToProcess.Reset();
		CurrentPackagesToProcess.Append(PackagesForNextLoop);
		PackagesForNextLoop.Reset();
	}
}

FName UAssetManager::GetEncryptionKeyAssetTagName()
{
	static const FName NAME_EncryptionKey("EncryptionKey");
	return NAME_EncryptionKey;
}

bool UAssetManager::ShouldScanPrimaryAssetType(FPrimaryAssetTypeInfo& TypeInfo) const
{
	if (!ensureMsgf(TypeInfo.PrimaryAssetType != PackageChunkType.GetName(), TEXT("Cannot use %s as an asset manager type, this is reserved for internal use"), *TypeInfo.PrimaryAssetType.ToString()))
	{
		// Cannot use this as a proper type
		return false;
	}

	if (TypeInfo.bIsEditorOnly && !GIsEditor)
	{
		return false;
	}

	bool bIsValid, bBaseClassWasLoaded;
	TypeInfo.FillRuntimeData(bIsValid, bBaseClassWasLoaded);

	if (bBaseClassWasLoaded)
	{
		// Had to load a class, mark that the temporary cache needs to be updated
		GetAssetRegistry().SetTemporaryCachingModeInvalidated();
	}

	return bIsValid;
}

void UAssetManager::ScanPrimaryAssetTypesFromConfig()
{
	SCOPED_BOOT_TIMING("UAssetManager::ScanPrimaryAssetTypesFromConfig");
	IAssetRegistry& AssetRegistry = GetAssetRegistry();
	const UAssetManagerSettings& Settings = GetSettings();

	PushBulkScanning();
	TGuardValue<bool> ScopeGuard(bScanningFromInitialConfig, true);

	double LastPumpTime = FPlatformTime::Seconds();
	for (FPrimaryAssetTypeInfo TypeInfo : Settings.PrimaryAssetTypesToScan)
	{
		// This function also fills out runtime data on the copy
		if (!ShouldScanPrimaryAssetType(TypeInfo))
		{
			continue;
		}

		UE_CLOGF(AssetTypeMap.Find(TypeInfo.PrimaryAssetType), LogAssetManager, Error, "Found multiple \"%ls\" Primary Asset Type entries in \"Primary Asset Types To Scan\" config. Only a single entry per type is supported.", *TypeInfo.PrimaryAssetType.ToString());

		ScanPathsForPrimaryAssets(TypeInfo.PrimaryAssetType, TypeInfo.AssetScanPaths, TypeInfo.AssetBaseClassLoaded, TypeInfo.bHasBlueprintClasses, TypeInfo.bIsEditorOnly, false);

		SetPrimaryAssetTypeRules(TypeInfo.PrimaryAssetType, TypeInfo.Rules);

		if (FPlatformTime::Seconds() > (LastPumpTime + 0.033f))
		{
			FPlatformApplicationMisc::PumpMessages(IsInGameThread());
			LastPumpTime = FPlatformTime::Seconds();
		}
	}

	PopBulkScanning();
}

void UAssetManager::ScanPrimaryAssetRulesFromConfig()
{
	const UAssetManagerSettings& Settings = GetSettings();

	// Read primary asset rule overrides
	for (const FPrimaryAssetRulesOverride& Override : Settings.PrimaryAssetRules)
	{
		if (Override.PrimaryAssetId.PrimaryAssetType == PrimaryAssetLabelType)
		{
			UE_LOGF(LogAssetManager, Error, "Cannot specify Rules overrides for Labels in ini! You most modify asset %ls!", *Override.PrimaryAssetId.ToString());
			continue;
		}

		SetPrimaryAssetRules(Override.PrimaryAssetId, Override.Rules);
	}

	for (const FPrimaryAssetRulesCustomOverride& Override : Settings.CustomPrimaryAssetRules)
	{
		ApplyCustomPrimaryAssetRulesOverride(Override);
	}
}

void UAssetManager::ApplyCustomPrimaryAssetRulesOverride(const FPrimaryAssetRulesCustomOverride& CustomOverride)
{
	TArray<FPrimaryAssetId> PrimaryAssets;
	GetPrimaryAssetIdList(CustomOverride.PrimaryAssetType, PrimaryAssets);

	for (FPrimaryAssetId PrimaryAssetId : PrimaryAssets)
	{
		if (DoesPrimaryAssetMatchCustomOverride(PrimaryAssetId, CustomOverride))
		{
			SetPrimaryAssetRules(PrimaryAssetId, CustomOverride.Rules);
		}
	}
}

bool UAssetManager::DoesPrimaryAssetMatchCustomOverride(FPrimaryAssetId PrimaryAssetId, const FPrimaryAssetRulesCustomOverride& CustomOverride) const
{
	if (!CustomOverride.FilterDirectory.Path.IsEmpty())
	{
		FSoftObjectPath AssetPath = GetPrimaryAssetPath(PrimaryAssetId);
		FString PathString = AssetPath.ToString();

		if (!PathString.Contains(CustomOverride.FilterDirectory.Path))
		{
			return false;
		}
	}

	// Filter string must be checked by an override of this function

	return true;
}

void UAssetManager::CallOrRegister_OnCompletedInitialScan(FSimpleMulticastDelegate::FDelegate&& Delegate)
{
	if (IsInitialized() && Get().HasInitialScanCompleted())
	{
		Delegate.Execute();
	}
	else
	{
		OnCompletedInitialScanDelegate.Add(MoveTemp(Delegate));
	}
}

void UAssetManager::CallOrRegister_OnAssetManagerCreated(FSimpleMulticastDelegate::FDelegate&& Delegate)
{
	if (IsInitialized())
	{
		Delegate.Execute();
	}
	else
	{
		OnAssetManagerCreatedDelegate.Add(MoveTemp(Delegate));
	}
}

bool UAssetManager::HasInitialScanCompleted() const
{
	return bHasCompletedInitialScan;
}

FDelegateHandle UAssetManager::Register_OnAddedAssetSearchRoot(FOnAddedAssetSearchRoot::FDelegate&& Delegate)
{
	return OnAddedAssetSearchRootDelegate.Add(MoveTemp(Delegate));
}

void UAssetManager::Unregister_OnAddedAssetSearchRoot(FDelegateHandle DelegateHandle)
{
	OnAddedAssetSearchRootDelegate.Remove(DelegateHandle);
}

bool UAssetManager::ExpandVirtualPaths(TArray<FString>& InOutPaths) const
{
	bool bMadeChange = false;
	for (int32 ReadIndex = 0; ReadIndex < InOutPaths.Num(); ReadIndex++)
	{
		const FString* PatternString = nullptr;
		const TArray<FString>* ReplacementStrings = nullptr;

		// Note, this does not support multiple virtual roots, which would conflict
		if (InOutPaths[ReadIndex].Contains(AssetSearchRootsVirtualPath))
		{
			PatternString = &AssetSearchRootsVirtualPath;
			ReplacementStrings = &GetAssetSearchRoots(true);
		}
		else if (InOutPaths[ReadIndex].Contains(DynamicSearchRootsVirtualPath))
		{
			PatternString = &DynamicSearchRootsVirtualPath;
			ReplacementStrings = &GetAssetSearchRoots(false);
		}
		
		if (PatternString)
		{
			bMadeChange = true;
			int32 NumReplacements = ReplacementStrings->Num();
			if (NumReplacements == 0)
			{
				// No replacements, just delete
				InOutPaths.RemoveAt(ReadIndex);
				ReadIndex--;
				continue;
			}

			// Add room for new strings and then replace
			FString ReadString = InOutPaths[ReadIndex];
			if (NumReplacements > 1)
			{
				InOutPaths.InsertDefaulted(ReadIndex + 1, NumReplacements - 1);
			}
			
			for (int32 ReplaceIndex = 0; ReplaceIndex < NumReplacements; ReplaceIndex++)
			{
				// This replacement is not case sensitive
				InOutPaths[ReadIndex + ReplaceIndex] = ReadString.Replace(**PatternString, *((*ReplacementStrings)[ReplaceIndex]));
			}

			// Deal with inserted strings and implicit ++ from for loop
			ReadIndex += (NumReplacements - 1);
		}
	}

	return bMadeChange;
}

void UAssetManager::AddAssetSearchRoot(const FString& NewRootPath)
{
	// Not valid to mount twice, or with case variation
	FString NormalizedPath = GetNormalizedPackagePath(NewRootPath, false);

	if (AllAssetSearchRoots.Contains(NormalizedPath))
	{
		UE_LOGF(LogAssetManager, Error, "AddAssetSearchRoot called twice with path %ls!", *NormalizedPath);
		return;
	}

	AllAssetSearchRoots.Add(NormalizedPath);
	AddedAssetSearchRoots.Add(NormalizedPath);

	OnAddedAssetSearchRootDelegate.Broadcast(NormalizedPath);
}

const TArray<FString>& UAssetManager::GetAssetSearchRoots(bool bIncludeStartupRoots) const
{
	return bIncludeStartupRoots ? AllAssetSearchRoots : AddedAssetSearchRoots;
}

void UAssetManager::PostInitialAssetScan()
{
	// Don't apply rules until scanning is done
	ScanPrimaryAssetRulesFromConfig();

	bIsPrimaryAssetDirectoryCurrent = true;

	if (!bHasCompletedInitialScan)
	{
		// Done with initial scan, fire delegate exactly once. This does not happen on editor refreshes
		bHasCompletedInitialScan = true;
		OnCompletedInitialScanDelegate.Broadcast();
		OnCompletedInitialScanDelegate.Clear();
	}

#if WITH_EDITOR
	if (bUpdateManagementDatabaseAfterScan)
	{
		bUpdateManagementDatabaseAfterScan = false;
		UpdateManagementDatabase(EUpdateManagementDatabaseFlags::BuildChunkMap | EUpdateManagementDatabaseFlags::ForceRefresh);
	}
#endif
}

bool UAssetManager::GetManagedPackageList(FPrimaryAssetId PrimaryAssetId, TArray<FName>& PackagePathList) const
{
	bool bFoundAny = false;
	TArray<FAssetIdentifier> FoundDependencies;
	TArray<FString> DependencyStrings;

	IAssetRegistry& AssetRegistry = GetAssetRegistry();
	AssetRegistry.GetDependencies(PrimaryAssetId, FoundDependencies, UE::AssetRegistry::EDependencyCategory::Manage);

	for (const FAssetIdentifier& Identifier : FoundDependencies)
	{
		if (Identifier.PackageName != NAME_None)
		{
			bFoundAny = true;
			PackagePathList.Add(Identifier.PackageName);
		}
	}
	return bFoundAny;
}

bool UAssetManager::GetPackageCookRuleManagers(FName PackageName, bool bRecurseToParents,
	TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Managers) const
{
	return GetPackageManagersInternal(PackageName, true /* bUseCookRuleReferences */,
		bRecurseToParents, Managers);
}

bool UAssetManager::GetPackageManagers(FName PackageName, bool bRecurseToParents,
	TSet<FPrimaryAssetId>& ManagerSet) const
{
	TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty> Managers;
	bool bFoundAny = GetPackageManagers(PackageName, bRecurseToParents, Managers);
	for (TPair<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Pair : Managers)
	{
		ManagerSet.Add(Pair.Key);
	}
	return bFoundAny;
}

bool UAssetManager::GetPackageManagers(FName PackageName, bool bRecurseToParents,
	TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Managers) const
{
	return GetPackageManagersInternal(PackageName, false /* bUseCookRuleReferences */,
		bRecurseToParents, Managers);
}

bool UAssetManager::GetPackageManagersInternal(FName PackageName,
	bool bUseCookRuleReferences, bool bRecurseToParents,
	TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Managers) const
{
	IAssetRegistry& AssetRegistry = GetAssetRegistry();
	auto UnionManageProperties = [](UE::AssetRegistry::EDependencyProperty A, UE::AssetRegistry::EDependencyProperty B)
	{
		return A | B;
	};

	// We have two ManagementParentMaps that are calculated using the two kinds of dependencies we track:
	// CookRule or Chunking. Pick the corresponding map based on bUseCookRuleReferencess.
	const TMap<FPrimaryAssetId, TArray<FPrimaryAssetId>>& UsedParentMap = bUseCookRuleReferences ?
		CookRuleManagementParentMap : ManagementParentMap;
	UE::AssetRegistry::EDependencyProperty RequiredProperties = bUseCookRuleReferences ?
		UE::AssetRegistry::EDependencyProperty::CookRule : UE::AssetRegistry::EDependencyProperty::None;

	bool bFoundAny = false;
	TArray<FAssetDependency> ReferencingPrimaryAssets;
	ReferencingPrimaryAssets.Reserve(128);

	UE::AssetRegistry::FDependencyQuery RequiredPropertiesQuery;
	RequiredPropertiesQuery.Required = RequiredProperties;
	AssetRegistry.GetReferencers(PackageName, ReferencingPrimaryAssets,
		UE::AssetRegistry::EDependencyCategory::Manage, RequiredPropertiesQuery);

	for (int32 IdentifierIndex = 0; IdentifierIndex < ReferencingPrimaryAssets.Num(); IdentifierIndex++)
	{
		FAssetDependency& AssetDependency = ReferencingPrimaryAssets[IdentifierIndex];
		FPrimaryAssetId PrimaryAssetId = AssetDependency.AssetId.GetPrimaryAssetId();
		if (!PrimaryAssetId.IsValid())
		{
			continue;
		}
		bFoundAny = true;
		UE::AssetRegistry::EDependencyProperty& ExistingProperties = Managers.FindOrAdd(
			PrimaryAssetId, UE::AssetRegistry::EDependencyProperty::None);
		ExistingProperties = UnionManageProperties(ExistingProperties, AssetDependency.Properties);

		if (bRecurseToParents)
		{
			const TArray<FPrimaryAssetId>* ManagementParents = UsedParentMap.Find(PrimaryAssetId);
			if (ManagementParents)
			{
				// AssetDependency can be invalidated because we are modifying the array it is in.
				// Copy its data now before we modify the array.
				UE::AssetRegistry::EDependencyCategory IndirectCategory = AssetDependency.Category;
				UE::AssetRegistry::EDependencyProperty IndirectProperties = AssetDependency.Properties;
				EnumRemoveFlags(IndirectProperties, UE::AssetRegistry::EDependencyProperty::Direct);
				for (const FPrimaryAssetId& Manager : *ManagementParents)
				{
					// Call FindOrAdd with -1 so we can use value != -1 to decide whether it already existed
					UE::AssetRegistry::EDependencyProperty& ExistingParent = Managers.FindOrAdd(Manager,
						(UE::AssetRegistry::EDependencyProperty)-1);
					if (ExistingParent == (UE::AssetRegistry::EDependencyProperty)-1)
					{
						ExistingParent = UE::AssetRegistry::EDependencyProperty::None;
						// Add to end of list to recurse into the parent.
						FAssetDependency& Added = ReferencingPrimaryAssets.Emplace_GetRef();
						Added.AssetId = Manager;
						Added.Category = IndirectCategory;
						// Set the parent's property equal to the child's properties, but change it to Indirect
						Added.Properties = IndirectProperties;
					}
				}
			}
		}
	}
	return bFoundAny;
}

void UAssetManager::StartInitialLoading()
{
	// The scan below queries asset registry, so we should make sure the premade registry is finished loading if it exists.
	GetAssetRegistry().WaitForPremadeAssetRegistry();

	GInitialBulkScan.StartOnce(this);

	ScanPrimaryAssetTypesFromConfig();

	OnAssetManagerCreatedDelegate.Broadcast();
	OnAssetManagerCreatedDelegate.Clear();
}

void UAssetManager::FinishInitialLoading()
{
	// See if we have pending scans, if so defer result
	bool bWaitingOnDeferredScan = false;

	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		if (TypeData.DeferredAssetScanPaths.Num())
		{
			bWaitingOnDeferredScan = true;
		}
	}

	if (!bWaitingOnDeferredScan)
	{
		PostInitialAssetScan();
	}
}

bool UAssetManager::IsPathExcludedFromScan(const FString& Path) const
{
	const UAssetManagerSettings& Settings = GetSettings();

	for (const FDirectoryPath& ExcludedPath : Settings.DirectoriesToExclude)
	{
		if (Path.Contains(ExcludedPath.Path))
		{
			return true;
		}
	}

	return false;
}

bool UAssetManager::IsScanningFromInitialConfig() const
{
	return bScanningFromInitialConfig;
}

bool UAssetManager::GetContentRootPathFromPackageName(const FString& PackageName, FString& OutContentRootPath)
{
	if (PackageName.StartsWith(TEXT("/"), ESearchCase::CaseSensitive))
	{
		const int32 SecondSlashIndex = PackageName.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromStart, 1);
		if (SecondSlashIndex != INDEX_NONE)
		{
			OutContentRootPath = PackageName.Mid(0, SecondSlashIndex + 1);
			return true;
		}
	}

	return false;
}

void UAssetManager::NormalizePackagePath(FString& InOutPath, bool bIncludeFinalSlash)
{
	InOutPath.ReplaceInline(TEXT("//"), TEXT("/"), ESearchCase::CaseSensitive);

	bool bEndsInSlash = InOutPath.EndsWith(TEXT("/"), ESearchCase::CaseSensitive);

	if (bIncludeFinalSlash && !bEndsInSlash)
	{
		InOutPath += TEXT("/");
	}
	else if (!bIncludeFinalSlash && bEndsInSlash)
	{
		InOutPath.LeftChopInline(1);
	}
}

FString UAssetManager::GetNormalizedPackagePath(const FString& InPath, bool bIncludeFinalSlash)
{
	FString ReturnPath = InPath;
	NormalizePackagePath(ReturnPath, bIncludeFinalSlash);
	return MoveTemp(ReturnPath);
}

void UAssetManager::WarnAboutInvalidPrimaryAsset(const FPrimaryAssetId& PrimaryAssetId, const FString& Message) const
{
	if (!WarningInvalidAssets.Contains(PrimaryAssetId))
	{
		WarningInvalidAssets.Add(PrimaryAssetId);

		const UAssetManagerSettings& Settings = GetSettings();
		if (Settings.bShouldWarnAboutInvalidAssets)
		{
			const TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetId.PrimaryAssetType);

			if (FoundType)
			{
				UE_LOGF(LogAssetManager, Warning, "Invalid Primary Asset Id %ls: %ls", *PrimaryAssetId.ToString(), *Message);
			}
			else
			{
				UE_LOGF(LogAssetManager, Warning, "Invalid Primary Asset Type %ls: %ls", *PrimaryAssetId.ToString(), *Message);
			}
		}
	}
}

void UAssetManager::InvalidatePrimaryAssetDirectory()
{
	bIsPrimaryAssetDirectoryCurrent = false;
}

void UAssetManager::RefreshPrimaryAssetDirectory(bool bForceRefresh)
{
	WarningInvalidAssets.Reset();

	// Do not refresh before the initial scan has completed
	if (!HasInitialScanCompleted())
	{
		return;
	}

	if (bForceRefresh || !bIsPrimaryAssetDirectoryCurrent)
	{
		PushBulkScanning();
		TGuardValue<bool> ScopeGuard(bScanningFromInitialConfig, true);

		for (TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
		{
			FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

			// Rescan the runtime data, the class may have gotten changed by hot reload or config changes
			bool bIsValid, bBaseClassWasLoaded;
			TypeData.Info.FillRuntimeData(bIsValid, bBaseClassWasLoaded);

			if (bBaseClassWasLoaded)
			{
				// Had to load a class, mark that the temporary cache needs to be updated
				GetAssetRegistry().SetTemporaryCachingModeInvalidated();
			}

			if (!bIsValid)
			{
				continue;
			}

			if (TypeData.Info.AssetScanPaths.Num())
			{
				// Clear old data if this type has actual scan paths
				TypeData.ResetAssets(AssetPathMap);

				// Rescan all assets. We don't force synchronous here as in the editor it was already loaded async
				ScanPathsForPrimaryAssets(TypePair.Key, TypeData.Info.AssetScanPaths, TypeData.Info.AssetBaseClassLoaded, TypeData.Info.bHasBlueprintClasses, TypeData.Info.bIsEditorOnly, false);
			}
		}

		PopBulkScanning();

		PostInitialAssetScan();
	}
}

#if WITH_EDITOR

EAssetSetManagerResult::Type UAssetManager::ShouldSetManager(UE::AssetRegistry::FShouldSetManagerContext& Context) const
{
	FPrimaryAssetId ManagerPrimaryAssetId = Context.Manager.GetPrimaryAssetId();
	FPrimaryAssetId TargetPrimaryAssetId = Context.Target.GetPrimaryAssetId();
	if (TargetPrimaryAssetId.IsValid())
	{
		// Don't recurse Primary Asset Id references
		return EAssetSetManagerResult::SetButDoNotRecurse;
	}

	// Ignore script references
	if (EnumHasAnyFlags(Context.TargetAssetFlags, UE::AssetRegistry::EAssetSetManagerAssetFlags::ScriptPackage))
	{
		return EAssetSetManagerResult::DoNotSet;
	}

	if (Context.EdgeFlags & EAssetSetManagerFlags::TargetHasExistingManager)
	{
		// If target has a higher priority manager, never recurse and only set manager if direct
		if (Context.EdgeFlags & EAssetSetManagerFlags::IsDirectSet)
		{
			return EAssetSetManagerResult::SetButDoNotRecurse;
		}
		else
		{
			return EAssetSetManagerResult::DoNotSet;
		}
	}
	else if (Context.EdgeFlags & EAssetSetManagerFlags::TargetHasDirectManager)
	{
		// If target has another direct manager being set in this run, never recurse and set manager if we think this is an "owner" reference and not a back reference

		bool bIsOwnershipReference = Context.EdgeFlags & EAssetSetManagerFlags::IsDirectSet;

		if (ManagerPrimaryAssetId.PrimaryAssetType == MapType)
		{
			// References made by maps are ownership references, because there is no way to distinguish between sublevels and top level maps we "include" sublevels in parent maps via reference
			bIsOwnershipReference = true;
		}

		if (bIsOwnershipReference)
		{
			return EAssetSetManagerResult::SetButDoNotRecurse;
		}
		else
		{
			return EAssetSetManagerResult::DoNotSet;
		}
	}
	return EAssetSetManagerResult::SetAndRecurse;
}

void UAssetManager::OnAssetRegistryFilesLoaded()
{
	PushBulkScanning();
	TGuardValue<bool> ScopeGuard(bScanningFromInitialConfig, true);

	for (TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		if (TypeData.DeferredAssetScanPaths.Num())
		{
			// File scan finished, now scan for assets. Maps are sorted so this will be in the order of original scan requests
			ScanPathsForPrimaryAssets(TypePair.Key, TypeData.DeferredAssetScanPaths.Array(), TypeData.Info.AssetBaseClassLoaded, TypeData.Info.bHasBlueprintClasses, TypeData.Info.bIsEditorOnly, false);

			TypeData.DeferredAssetScanPaths.Empty();
		}
	}

	PopBulkScanning();

	PostInitialAssetScan();
}

void UAssetManager::UpdateManagementDatabase(bool bForceRefresh)
{
	EUpdateManagementDatabaseFlags UpdateFlags = EUpdateManagementDatabaseFlags::BuildChunkMap;
	if (bForceRefresh)
	{
		UpdateFlags |= EUpdateManagementDatabaseFlags::ForceRefresh;
	}
	UpdateManagementDatabase(UpdateFlags);
}

void UAssetManager::UpdateManagementDatabase(EUpdateManagementDatabaseFlags InFlags)
{
	if (!GIsEditor)
	{
		// Doesn't work in standalone game because we haven't scanned all the paths
		UE_LOGF(LogAssetManager, Error, "UpdateManagementDatabase does not work in standalone game because it doesn't load the entire Asset Registry!");
	}

	// Construct the asset management map and pass it to the asset registry
	IAssetRegistry& AssetRegistry = GetAssetRegistry();

	if (AssetRegistry.IsGathering())
	{
		bUpdateManagementDatabaseAfterScan = true;
		return;
	}

	const bool bForceRefresh = EnumHasAnyFlags(InFlags, EUpdateManagementDatabaseFlags::ForceRefresh);
	if (bIsManagementDatabaseCurrent && !bForceRefresh)
	{
		return;
	}
	LLM_SCOPE_BYTAG(AssetManager);

	CookRuleManagementParentMap.Reset();
	ManagementParentMap.Reset();

	// Make sure the asset labels are up to date 
	ApplyPrimaryAssetLabels();

	// Map from Priority to map, then call in order
	TMap<int32, TMultiMap<FAssetIdentifier, FAssetIdentifier> > PriorityManagementMap;

	// List of references to not recurse on, priority doesn't matter
	TMultiMap<FAssetIdentifier, FAssetIdentifier> NoReferenceManagementMap;

	// List of packages that need to have their chunks updated
	const UProjectPackagingSettings* ProjectPackagingSettings = GetDefault<UProjectPackagingSettings>();
	const bool bWillBuildChunkMap = ProjectPackagingSettings && ProjectPackagingSettings->bGenerateChunks && EnumHasAnyFlags(InFlags, EUpdateManagementDatabaseFlags::BuildChunkMap);
	TSet<FName> PackagesToUpdateChunksFor;
	TArray<FName> AssetPackagesReferenced;

	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		for (const TPair<FName, FPrimaryAssetData>& NamePair : TypeData.GetAssets())
		{
			const FPrimaryAssetData& NameData = NamePair.Value;
			FPrimaryAssetId PrimaryAssetId(TypePair.Key, NamePair.Key);

			FPrimaryAssetRules Rules = GetPrimaryAssetRules(PrimaryAssetId);

			// Get the list of directly referenced assets, the registry wants it as FNames
			AssetPackagesReferenced.Reset();

			const FSoftObjectPath& AssetRef = NameData.GetAssetPtr().ToSoftObjectPath();

			if (AssetRef.IsValid())
			{
				FName PackageName = FName(*AssetRef.GetLongPackageName());

				if (PackageName == NAME_None)
				{
					UE_LOGF(LogAssetManager, Warning, "Ignoring 'None' reference originating from %ls from NameData", *PrimaryAssetId.ToString());
				}
				else
				{
					AssetPackagesReferenced.Add(PackageName);
					if (bWillBuildChunkMap)
					{
						PackagesToUpdateChunksFor.Add(PackageName);
					}
				}
			}


			// Add bundle references to manual reference list
			if (const TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>* BundleMap = CachedAssetBundles.Find(PrimaryAssetId))
			{
				for (const FAssetBundleEntry& Entry : (**BundleMap).Bundles)
				{
					for (const FTopLevelAssetPath& BundleAssetRef : Entry.AssetPaths)
					{
						FName PackageName = BundleAssetRef.GetPackageName();

						if (PackageName.IsNone())
						{
							UE_LOGF(LogAssetManager, Warning, "Ignoring 'None' reference originating from %ls from Bundle %ls", *PrimaryAssetId.ToString(), *PrimaryAssetId.ToString());
						}
						else
						{
							AssetPackagesReferenced.Add(PackageName);
							if (bWillBuildChunkMap)
							{
								PackagesToUpdateChunksFor.Add(PackageName);
							}
						}
					}
				}
			}

			Algo::Sort(AssetPackagesReferenced, FNameLexicalLess());
			AssetPackagesReferenced.SetNum(Algo::Unique(AssetPackagesReferenced), EAllowShrinking::No);
			for (const FName& AssetPackage : AssetPackagesReferenced)
			{
				TMultiMap<FAssetIdentifier, FAssetIdentifier>& ManagerMap = Rules.bApplyRecursively ? PriorityManagementMap.FindOrAdd(Rules.Priority) : NoReferenceManagementMap;

				ManagerMap.Add(PrimaryAssetId, AssetPackage);
			}
		}
	}

	TArray<int32> PriorityArray;
	PriorityManagementMap.GenerateKeyArray(PriorityArray);

	// Sort to highest priority first
	PriorityArray.Sort([](const int32& LHS, const int32& RHS) { return LHS > RHS; });

	FScopedSlowTask SlowTask(PriorityArray.Num(), LOCTEXT("BuildingManagementDatabase", "Building Asset Management Database"));
	const bool bShowCancelButton = false;
	const bool bAllowInPIE = true;
	SlowTask.MakeDialog(bShowCancelButton, bAllowInPIE);

	auto SetManagerPredicate = [this, bWillBuildChunkMap, &PackagesToUpdateChunksFor](UE::AssetRegistry::FShouldSetManagerContext& Context)
		{
			EAssetSetManagerResult::Type Result = this->ShouldSetManager(Context);
			if (bWillBuildChunkMap && Result != EAssetSetManagerResult::DoNotSet && Context.Target.IsPackage())
			{
				PackagesToUpdateChunksFor.Add(Context.Target.PackageName);
			}
			return Result;
		};
	auto SetManagerPredicateDirectSet = [](UE::AssetRegistry::FShouldSetManagerContext& Context)
		{
			return EAssetSetManagerResult::SetButDoNotRecurse;
		};

	UE::AssetRegistry::FSetManageReferencesContext SetManageContext;
	
	for (int32 PriorityIndex = 0; PriorityIndex < PriorityArray.Num(); PriorityIndex++)
	{
		SetManageContext.ManagerMap = PriorityManagementMap.Find(PriorityArray[PriorityIndex]);
		SetManageContext.bClearExisting = PriorityIndex == 0;
		SetManageContext.RecurseType = UE::AssetRegistry::EDependencyCategory::Package;
		SetManageContext.ShouldSetManager = SetManagerPredicate;
		SlowTask.EnterProgressFrame(1);

		AssetRegistry.SetManageReferences(SetManageContext);
	}

	SetManageContext.bClearExisting = false;
	SetManageContext.RecurseType = UE::AssetRegistry::EDependencyCategory::None;
	SetManageContext.ShouldSetManager = SetManagerPredicateDirectSet;

	// Do non recursive set last
	if (NoReferenceManagementMap.Num() > 0)
	{
		SetManageContext.ManagerMap = &NoReferenceManagementMap;
		AssetRegistry.SetManageReferences(SetManageContext);
	}

	TMultiMap<FAssetIdentifier, FAssetIdentifier> PrimaryAssetIdManagementMap;
	TArray<int32> ChunkList;
	TArray<int32> ExistingChunkList;

	// Update management parent list, which is PrimaryAssetId -> PrimaryAssetId
	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		for (const TPair<FName, FPrimaryAssetData>& NamePair : TypeData.GetAssets())
		{
			const FPrimaryAssetData& NameData = NamePair.Value;
			FPrimaryAssetId PrimaryAssetId(TypePair.Key, NamePair.Key);
			const FSoftObjectPath& AssetRef = NameData.GetAssetPtr().ToSoftObjectPath();

			// ChunkManagers are a superset of CookRuleManagers; we rely on that to avoid the need
			// for querying CookRuleManagers when Managers is empty.
			TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty> Managers;

			if (AssetRef.IsValid())
			{
				FName PackageName = FName(*AssetRef.GetLongPackageName());

				if (GetPackageManagers(PackageName, false, Managers) && Managers.Num() > 1)
				{
					// Find all managers that aren't this specific asset
					for (const TPair<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& ManagerPair : Managers)
					{
						if (ManagerPair.Key != PrimaryAssetId)
						{
							// Update the cached version and the version in registry
							if (EnumHasAnyFlags(ManagerPair.Value, UE::AssetRegistry::EDependencyProperty::CookRule))
							{
								CookRuleManagementParentMap.FindOrAdd(PrimaryAssetId).AddUnique(ManagerPair.Key);
							}
							ManagementParentMap.FindOrAdd(PrimaryAssetId).AddUnique(ManagerPair.Key);

							PrimaryAssetIdManagementMap.Add(ManagerPair.Key, PrimaryAssetId);
						}
					}
				}
			}
			else
			{
				Managers.Add(PrimaryAssetId, UE::AssetRegistry::EDependencyProperty::ManageMask);
			}

			// Compute chunk assignments for the PrimaryAsset and record them as manage references from a
			// PrimaryAssetId we create for each chunk
			ChunkList.Reset();
			TSet<FPrimaryAssetId> ManagersSet;
			Managers.GetKeys(ManagersSet);
			GetPrimaryAssetSetChunkIds(ManagersSet, nullptr, ExistingChunkList, ChunkList);

			for (int32 ChunkId : ChunkList)
			{
				FPrimaryAssetId ChunkPrimaryAsset = CreatePrimaryAssetIdFromChunkId(ChunkId);
				PrimaryAssetIdManagementMap.Add(ChunkPrimaryAsset, PrimaryAssetId);
			}
		}
	}

	if (PrimaryAssetIdManagementMap.Num() > 0)
	{
		SetManageContext.ManagerMap = &PrimaryAssetIdManagementMap;
		AssetRegistry.SetManageReferences(SetManageContext);
	}
	
	if (bWillBuildChunkMap)
	{
		CachedChunkMap = BuildChunkMap(PackagesToUpdateChunksFor, {}, {});
	}	
	bIsManagementDatabaseCurrent = true;
}

const TMap<int32, FAssetManagerChunkInfo>& UAssetManager::GetChunkManagementMap() const
{
	return CachedChunkMap;
}

void UAssetManager::ApplyPrimaryAssetLabels()
{
	// Load all of them off disk. Turn off soft object path tracking to avoid them getting cooked
	FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None, ESoftObjectPathCollectType::NeverCollect, ESoftObjectPathSerializeType::AlwaysSerialize);

	TSharedPtr<FStreamableHandle> Handle = LoadPrimaryAssetsWithType(PrimaryAssetLabelType);

	if (Handle.IsValid())
	{
		Handle->WaitUntilComplete();
	}
	
	// PostLoad in PrimaryAssetLabel sets PrimaryAssetRules overrides
}

void UAssetManager::ModifyCook(TConstArrayView<const ITargetPlatform*> TargetPlatforms, TArray<FName>& PackagesToCook, TArray<FName>& PackagesToNeverCook)
{
	check(TargetPlatforms.Num() > 0);
	bTargetPlatformsAllowDevelopmentObjects = TargetPlatforms[0]->AllowsDevelopmentObjects();
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms.Slice(1,TargetPlatforms.Num() - 1))
	{
		if (TargetPlatform->AllowsDevelopmentObjects() != bTargetPlatformsAllowDevelopmentObjects)
		{
			const ITargetPlatform* PlatformThatDoesNotAllow =
				bTargetPlatformsAllowDevelopmentObjects ? TargetPlatform : TargetPlatforms[0];
			UE_LOGF(LogAssetManager, Error,
				"Cooking platform %ls and %ls in a single cook is not supported, because they have different values for AllowsDevelopmentObjects. " "This cook session will use AllowsDevelopmentObjects = true, which will add packages to platform %ls that should not be present.",
				*TargetPlatforms[0]->PlatformName(), *TargetPlatform->PlatformName(),
				*PlatformThatDoesNotAllow->PlatformName());
			bTargetPlatformsAllowDevelopmentObjects = true;
			break;
		}
	}
	// Make sure management database is set up
	UpdateManagementDatabase();

	// Cook all non-editor types
	TArray<FPrimaryAssetTypeInfo> TypeList;

	GetPrimaryAssetTypeInfoList(TypeList);

	bool bIncludeDevelopmentAssets = !bOnlyCookProductionAssets || bTargetPlatformsAllowDevelopmentObjects;

	// Some primary assets exist in the transient package. No need to include them in the cook since they are transient.
	FName TransientPackageName = GetTransientPackage()->GetFName();

	// Uniquely append packages we need that are not already in PackagesToCook and PackagesToNeverCook
	TSet<FName> PackagesToCookSet(PackagesToCook);
	TSet<FName> PackagesToNeverCookSet(PackagesToNeverCook);

	// Get package names in the libraries that we care about for cooking. Only get ones that are needed in production
	TArray<FName> AssetPackages;
	for (const FPrimaryAssetTypeInfo& TypeInfo : TypeList)
	{
		// Cook these types
		TArray<FPrimaryAssetId> AssetIdList;
		GetPrimaryAssetIdList(TypeInfo.PrimaryAssetType, AssetIdList);
		AssetPackages.Reset();

		for (const FPrimaryAssetId& PrimaryAssetId : AssetIdList)
		{
			FAssetData AssetData;
			if (GetPrimaryAssetData(PrimaryAssetId, AssetData) && AssetData.PackageName != TransientPackageName)
			{
				// If this has an asset data, add that package name
				AssetPackages.Add(AssetData.PackageName);
			}

			// Also add any bundle assets to handle cook rules for labels
			TArray<FAssetBundleEntry> FoundEntries;
			if (GetAssetBundleEntries(PrimaryAssetId, FoundEntries))
			{
				for (const FAssetBundleEntry& FoundEntry : FoundEntries)
				{
					for (const FTopLevelAssetPath& FoundReference : FoundEntry.AssetPaths)
					{
						FName PackageName = FoundReference.GetPackageName();
						AssetPackages.Add(PackageName);
					}
				}
			}
		}
		Algo::Sort(AssetPackages, FNameFastLess());
		AssetPackages.SetNum(Algo::Unique(AssetPackages), EAllowShrinking::No);

		for (FName PackageName : AssetPackages)
		{
			EPrimaryAssetCookRule CookRule = GetPackageCookRule(PackageName);
			bool bAlwaysCook = CookRule == EPrimaryAssetCookRule::AlwaysCook ||
				(bIncludeDevelopmentAssets && (
					CookRule == EPrimaryAssetCookRule::DevelopmentAlwaysProductionUnknownCook ||
					CookRule == EPrimaryAssetCookRule::DevelopmentAlwaysProductionNeverCook));
			bool bCanCook = VerifyCanCookPackage(nullptr, PackageName, false);

			if (bAlwaysCook && bCanCook && !TypeInfo.bIsEditorOnly)
			{
				// If this is always cook, not excluded, and not editor only, cook it
				bool bAlreadyInSet;
				PackagesToCookSet.Add(PackageName, &bAlreadyInSet);
				if (!bAlreadyInSet)
				{
					PackagesToCook.Add(PackageName);
				}
			}
			else if (!bCanCook)
			{
				// If this package cannot be cooked, add to exclusion list
				bool bAlreadyInSet;
				PackagesToNeverCookSet.Add(PackageName, &bAlreadyInSet);
				if (!bAlreadyInSet)
				{
					PackagesToNeverCook.Add(PackageName);
				}
			}
		}
	}
}

void UAssetManager::ModifyDLCCook(const FString& DLCName, TConstArrayView<const ITargetPlatform*> TargetPlatforms,
	TArray<FName>& PackagesToCook, TArray<FName>& PackagesToNeverCook)
{
	UE_LOGF(LogAssetManager, Display, "ModifyDLCCook: Scanning Plugin Directory %ls for assets, and adding them to the cook list", *DLCName);
	FString DLCPath;
	FString ExternalMountPointName;
	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(DLCName))
	{
		DLCPath = Plugin->GetContentDir();
		ExternalMountPointName = Plugin->GetMountedAssetPath();
	}
	else
	{
		DLCPath = FPaths::ProjectPluginsDir() / DLCName / TEXT("Content");
		ExternalMountPointName = FString::Printf(TEXT("/%s/"), *DLCName);
	}

	TArray<FString> Files;
	IFileManager::Get().FindFilesRecursive(Files, *DLCPath, *(FString("*") + FPackageName::GetAssetPackageExtension()), true, false, false);
	IFileManager::Get().FindFilesRecursive(Files, *DLCPath, *(FString("*") + FPackageName::GetMapPackageExtension()), true, false, false);
	for (const FString& CurrentFile : Files)
	{
		const FString StdFile = FPaths::CreateStandardFilename(CurrentFile);
		PackagesToCook.AddUnique(FName(StdFile));
		FString LongPackageName;
		if (!FPackageName::IsValidLongPackageName(StdFile) && !FPackageName::TryConvertFilenameToLongPackageName(StdFile, LongPackageName))
		{
			FPackageName::RegisterMountPoint(ExternalMountPointName, DLCPath);
		}
	}
}

void UAssetManager::ModifyCookReferences(FName PackageName, TArray<FName>& PackagesToCook)
{
	TArray<FTopLevelAssetPath>* Paths = AssetBundlePathsForPackage.Find(PackageName);
	if (!Paths)
	{
		return;
	}
	PackagesToCook.Reserve(Paths->Num());
	for (const FTopLevelAssetPath& Path : *Paths)
	{
		PackagesToCook.Add(Path.GetPackageName());
	}
	Algo::Sort(PackagesToCook, FNameFastLess());
	PackagesToCook.SetNum(Algo::Unique(PackagesToCook));
}

void UAssetManager::GatherPublicAssetsForPackage(FName PackagePath, TSet<FSoftObjectPath>& AssetsToCook) const
{
	FARFilter Filter;
	Filter.PackagePaths.Add(PackagePath);
	Filter.bRecursivePaths = true;	
	Filter.bIncludeOnlyOnDiskAssets = true;
	Filter.WithoutPackageFlags |= PKG_NotExternallyReferenceable;

	GetAssetRegistry().EnumerateAssets(Filter, [&AssetsToCook](const FAssetData& AssetData)
	{
		// Only TopLevelAssets can be API exposed
		if (AssetData.IsTopLevelAsset())
		{
			// this package can be externally referenced; include it in the cook
			if (!AssetData.PackageName.IsNone())
			{
				UE_LOGF(LogAssetManager, Verbose,
					"GatherPublicAssetsForPackage: Adding public package [%ls] (instigator: [%ls]",
					*AssetData.PackageName.ToString(),
					*AssetData.AssetName.ToString());
				AssetsToCook.Add(AssetData.GetSoftObjectPath());
			}
			else
			{
				UE_LOGF(LogAssetManager, Error,
					"GatherPublicAssetsForPackage: Failed to resolve package for asset [%ls]",
					*AssetData.AssetName.ToString());
			}
		}

		return true;
	});
}

void UAssetManager::GatherPublicAssetsForPackage(FName PackagePath, TArray<FName>& PackagesToCook) const
{
	TSet<FSoftObjectPath> AssetsToCook;
	GatherPublicAssetsForPackage(PackagePath, AssetsToCook);

	if (AssetsToCook.Num() > 0)
	{
		TSet<FName> PackagesToCookSet;
		PackagesToCookSet.Reserve(PackagesToCook.Num() + AssetsToCook.Num());
		PackagesToCookSet.Append(PackagesToCook);

		PackagesToCook.Reserve(PackagesToCook.Num() + AssetsToCook.Num());
		for (const FSoftObjectPath& AssetToCook : AssetsToCook)
		{
			const FName PackageToCook = AssetToCook.GetLongPackageFName();
			bool bAlreadyInSet = false;
			PackagesToCookSet.Add(PackageToCook, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				PackagesToCook.Add(PackageToCook);
			}
		}
	}
}

bool UAssetManager::ShouldCookForPlatform(const UPackage* Package, const ITargetPlatform* TargetPlatform)
{
	return true;
}

EPrimaryAssetCookRule UAssetManager::GetPackageCookRule(FName PackageName) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetManager::GetPackageCookRule);

	TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty> Managers;
	GetPackageCookRuleManagers(PackageName, true, Managers);

	TOptional<TTuple<FPrimaryAssetId, FPrimaryAssetId>> ConflictIds;
	EPrimaryAssetCookRule CookRule = CalculateCookRuleUnion(Managers, &ConflictIds);
	if (ConflictIds)
	{
		UE_LOGF(LogAssetManager, Error, "GetPackageCookRule: Conflicting Cook Rule for package %ls! %ls and %ls have the same priority and disagree.",
			*PackageName.ToString(), *ConflictIds->Get<0>().ToString(), *ConflictIds->Get<1>().ToString());
	}

	return CookRule;
}

EPrimaryAssetCookRule UAssetManager::CalculateCookRuleUnion(const TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Managers,
	TOptional<TPair<FPrimaryAssetId, FPrimaryAssetId>>* OutConflictIds) const
{
	FPrimaryAssetCookRuleUnion Union;
	for (const TPair<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Pair : Managers)
	{
		FPrimaryAssetRules Rules = GetPrimaryAssetRules(Pair.Key);
		bool bDirect = EnumHasAllFlags(Pair.Value, UE::AssetRegistry::EDependencyProperty::Direct);
		Union.UnionWith(Rules.CookRule, bDirect, Pair.Key, Rules.Priority);
	}

	return Union.GetRule(OutConflictIds);
}

void FPrimaryAssetCookRuleUnion::UnionWith(EPrimaryAssetCookRule CookRule, bool bDirectReference, const FPrimaryAssetId& Id, int32 Priority)
{
	auto MarkExcluded = [this, bDirectReference, &Id, Priority](EPrimaryAssetProductionLevel LowestLevelToExclude)
	{
		// Exclusion only applies to direct references
		if (bDirectReference)
		{
			for (int32 LevelInt = (int32)LowestLevelToExclude; LevelInt < (int32)EPrimaryAssetProductionLevel::Count; ++LevelInt)
			{
				FAssignmentInfo& Info = ExclusionByLevel[LevelInt];
				if (!Info.bSet || Info.Priority < Priority)
				{
					Info.bSet = true;
					Info.Priority = Priority;
					Info.Id = Id;
				}
			}
		}
	};
	auto MarkIncluded = [this, &Id, Priority](EPrimaryAssetProductionLevel HighestLevelToInclude)
	{
		// Referenced applies to direct and indirect references
		for (int32 LevelInt = 0; LevelInt <= (int32)HighestLevelToInclude; ++LevelInt)
		{
			FAssignmentInfo& Info = InclusionByLevel[LevelInt];
			if (!Info.bSet || Info.Priority < Priority)
			{
				Info.bSet = true;
				Info.Priority = Priority;
				Info.Id = Id;
			}
		}
	};

	switch (CookRule)
	{
	case EPrimaryAssetCookRule::Unknown:
		// Managers with CookRule Unknown are only used to define Chunks for assets that other managers include
		// and do not affect whether the Asset should be cooked
		break;
	case EPrimaryAssetCookRule::NeverCook:
		// Managers with NeverCook require that the asset is NOT cooked in production or development builds,
		// but only for direct references
		MarkExcluded(EPrimaryAssetProductionLevel::Development);
		break;
	case EPrimaryAssetCookRule::ProductionNeverCook:
		// Managers with ProductionNeverCook (1) do not imply the asset should be cooked for development but
		// (2) DO require that the asset is NOT cooked in production builds, but only for direct references
		MarkExcluded(EPrimaryAssetProductionLevel::Production);
		break;
	case EPrimaryAssetCookRule::DevelopmentAlwaysProductionNeverCook:
		// Managers with DevelopmentAlwaysProductionNeverCook (1) require the asset should be cooked in development builds
		// (2) require that the asset is NOT cooked in production builds, but only for direct references
		MarkIncluded(EPrimaryAssetProductionLevel::Development);
		MarkExcluded(EPrimaryAssetProductionLevel::Production);
		break;
	case EPrimaryAssetCookRule::DevelopmentAlwaysProductionUnknownCook:
		// Managers with DevelopmentAlwaysProductionUnknownCook (1) require the asset should be cooked in development builds
		// (2) imply neither inclusion nor exclusion for production builds
		MarkIncluded(EPrimaryAssetProductionLevel::Development);
		break;
	case EPrimaryAssetCookRule::AlwaysCook:
		// Managers with AlwaysCook (1) require the asset should be cooked in production and development
		MarkIncluded(EPrimaryAssetProductionLevel::Production);
		break;
	default:
		checkNoEntry();
		break;
	}
}

EPrimaryAssetCookRule FPrimaryAssetCookRuleUnion::GetRule(TOptional<TTuple<FPrimaryAssetId, FPrimaryAssetId>>* OutConflictId)
{
	if (OutConflictId)
	{
		OutConflictId->Reset();
	}

	FAssignmentInfo& ProdInclusion= InclusionByLevel[(int32)EPrimaryAssetProductionLevel::Production];
	FAssignmentInfo& ProdExclusion= ExclusionByLevel[(int32)EPrimaryAssetProductionLevel::Production];
	FAssignmentInfo& DevInclusion = InclusionByLevel[(int32)EPrimaryAssetProductionLevel::Development];
	FAssignmentInfo& DevExclusion = ExclusionByLevel[(int32)EPrimaryAssetProductionLevel::Development];
	if (DevExclusion.bSet && (!DevInclusion.bSet || DevInclusion.Priority <= DevExclusion.Priority))
	{
		if (DevInclusion.bSet && DevInclusion.Priority == DevExclusion.Priority)
		{
			OutConflictId->Emplace(DevExclusion.Id, DevInclusion.Id);
		}
		return EPrimaryAssetCookRule::NeverCook;
	}

	if (ProdExclusion.bSet && (!ProdInclusion.bSet || ProdInclusion.Priority <= ProdExclusion.Priority))
	{
		if (ProdInclusion.bSet && ProdInclusion.Priority == ProdExclusion.Priority)
		{
			OutConflictId->Emplace(ProdExclusion.Id, ProdInclusion.Id);
		}

		if (!DevInclusion.bSet)
		{
			return EPrimaryAssetCookRule::ProductionNeverCook;
		}
		else
		{
			return EPrimaryAssetCookRule::DevelopmentAlwaysProductionNeverCook;
		}
	}

	if (!DevInclusion.bSet)
	{
		return EPrimaryAssetCookRule::Unknown;
	}
	else if (!ProdInclusion.bSet)
	{
		return EPrimaryAssetCookRule::DevelopmentAlwaysProductionUnknownCook;
	}
	else
	{
		return EPrimaryAssetCookRule::AlwaysCook;
	}
}

static FString GetInstigatorChainString(UE::Cook::ICookInfo* CookInfo, FName PackageName)
{
	if (!CookInfo)
	{
		return FString("<NoCookInfo>");
	}
	TArray<UE::Cook::FInstigator> Chain = CookInfo->GetInstigatorChain(PackageName);
	TStringBuilder<1024> Result;
	bool bFirst = true;
	for (const UE::Cook::FInstigator& Instigator : Chain)
	{
		Result << (bFirst ? TEXT("") : TEXT(" <- "));
		bFirst = false;
		Result << TEXT("{ ") << Instigator.ToString() << TEXT(" }");
	}
	return FString(Result);
};

bool UAssetManager::VerifyCanCookPackage(UE::Cook::ICookInfo* CookInfo, FName PackageName, bool bLogError) const
{
	bool bRetVal = true;
	EPrimaryAssetCookRule CookRule = GetPackageCookRule(PackageName);
	if (CookRule == EPrimaryAssetCookRule::NeverCook)
	{
		if (bLogError)
		{
			UE_LOGF(LogAssetManager, Error, "Package %ls is set to NeverCook, but something is trying to cook it! Instigators: %ls",
				*PackageName.ToString(), *GetInstigatorChainString(CookInfo, PackageName));
		}
		
		bRetVal = false;
	}
	else if ((CookRule == EPrimaryAssetCookRule::ProductionNeverCook || CookRule == EPrimaryAssetCookRule::DevelopmentAlwaysProductionNeverCook)
		&& bOnlyCookProductionAssets && !bTargetPlatformsAllowDevelopmentObjects)
	{
		if (bLogError)
		{
			UE_LOGF(LogAssetManager, Warning, "Package %ls is set to ProductionNeverCook, and bOnlyCookProductionAssets is true, but something is trying to cook it! Instigators: %ls",
				*PackageName.ToString(), *GetInstigatorChainString(CookInfo, PackageName));
		}

		bRetVal = false;
	}

	if (!bRetVal && bLogError)
	{
		TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty> Managers;
		GetPackageCookRuleManagers(PackageName, true, Managers);
		UE_LOGF(LogAssetManager, Display, "Listing Managers... (Count:%d)", Managers.Num());
		for (const TPair<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& PrimaryAssetPair : Managers)
		{
			UE_LOGF(LogAssetManager, Display, "  %ls", *PrimaryAssetPair.Key.ToString());
		}

		TArray<FName> PackageReferencers;
		GetAssetRegistry().GetReferencers(PackageName, PackageReferencers, UE::AssetRegistry::EDependencyCategory::Package);
		UE_LOGF(LogAssetManager, Display, "Listing known direct referencers... (Count:%d)", PackageReferencers.Num());
		for (FName PackageReferencer : PackageReferencers)
		{
			UE_LOGF(LogAssetManager, Display, "  %ls", *PackageReferencer.ToString());
		}
	}

	return bRetVal;
}

bool UAssetManager::GetPackageChunkIds(FName PackageName, const ITargetPlatform* TargetPlatform, TArrayView<const int32> ExistingChunkList, TArray<int32>& OutChunkList, TArray<int32>* OutOverrideChunkList) const
{
	// Include preset chunks
	OutChunkList.Append(ExistingChunkList.GetData(), ExistingChunkList.Num());
	if (OutOverrideChunkList)
	{
		OutOverrideChunkList->Append(ExistingChunkList.GetData(), ExistingChunkList.Num());
	}

	if (PackageName.ToString().StartsWith(TEXT("/Engine/"), ESearchCase::CaseSensitive))
	{
		// Some engine content is only referenced by string, make sure it's all in chunk 0 to avoid issues
		OutChunkList.AddUnique(0);

		if (OutOverrideChunkList)
		{
			OutOverrideChunkList->AddUnique(0);
		}
	}

	// Add all chunk ids from the asset rules of managers. By default priority will not override other chunks
	TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty> Managers;
	Managers.Reserve(128);

	GetPackageManagers(PackageName, true, Managers);
	TSet<FPrimaryAssetId> ManagersSet;
	Managers.GetKeys(ManagersSet);
	return GetPrimaryAssetSetChunkIds(ManagersSet, TargetPlatform, ExistingChunkList, OutChunkList);
}

void UAssetManager::UpdateCachedChunkMapAfterCook(const TSet<FName>& AllCookedPackages, const TSet<FName>& StartupPackages)
{
	CachedChunkMap = BuildChunkMap(AllCookedPackages, StartupPackages, AllCookedPackages);
}

bool UAssetManager::GetPrimaryAssetSetChunkIds(const TSet<FPrimaryAssetId>& PrimaryAssetSet, const class ITargetPlatform* TargetPlatform,
	TArrayView<const int32> ExistingChunkList, TArray<int32>& OutChunkList) const
{
	return GetPrimaryAssetSetChunkIds(PrimaryAssetSet.Array(), TargetPlatform, ExistingChunkList, OutChunkList);
}

bool UAssetManager::GetPrimaryAssetSetChunkIds(TConstArrayView<FPrimaryAssetId> PrimaryAssets, const class ITargetPlatform* TargetPlatform,
	TArrayView<const int32> ExistingChunkList, TArray<int32>& OutChunkList) const
{
	bool bFoundAny = false;
	int32 HighestChunk = 0;
	for (const FPrimaryAssetId& PrimaryAssetId : PrimaryAssets)
	{
		FPrimaryAssetRules Rules = GetPrimaryAssetRules(PrimaryAssetId);

		if (Rules.ChunkId != INDEX_NONE)
		{
			bFoundAny = true;
			OutChunkList.AddUnique(Rules.ChunkId);

			if (Rules.ChunkId > HighestChunk)
			{
				HighestChunk = Rules.ChunkId;
			}
		}
	}

	// Use chunk dependency info to remove redundant chunks
	UChunkDependencyInfo* DependencyInfo = GetMutableDefault<UChunkDependencyInfo>();
	DependencyInfo->GetOrBuildChunkDependencyGraph(HighestChunk);
	DependencyInfo->RemoveRedundantChunks(OutChunkList);

	return bFoundAny;
}

void UAssetManager::PreBeginPIE(bool bStartSimulate)
{
	if (HasInitialScanCompleted())
	{
		// If the scan has finished, we need to refresh in case there have been in-editor changes
		RefreshPrimaryAssetDirectory();
	}
	else
	{
		// If the scan is still in progress, we need to finish it now which will call PostInitialAssetScan
		GetAssetRegistry().WaitForCompletion();

		ensure(HasInitialScanCompleted());
	}

	// Cache asset state
	GetPrimaryAssetBundleStateMap(PrimaryAssetStateBeforePIE, false);
}

void UAssetManager::EndPIE(bool bStartSimulate)
{
	// Reset asset load state
	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		for (const TPair<FName, FPrimaryAssetData>& NamePair : TypeData.GetAssets())
		{
			const FPrimaryAssetData& NameData = NamePair.Value;
			const FPrimaryAssetLoadState& LoadState = (!NameData.PendingState.IsValid()) ? NameData.CurrentState : NameData.PendingState;

			if (!LoadState.IsValid())
			{
				// Don't worry about things that aren't loaded
				continue;
			}

			FPrimaryAssetId AssetID(TypePair.Key, NamePair.Key);

			TArray<FName>* BundleState = PrimaryAssetStateBeforePIE.Find(AssetID);

			if (BundleState)
			{
				// This will reset state to what it was before
				LoadPrimaryAsset(AssetID, *BundleState);
			}
			else
			{
				// Not in map, unload us
				UnloadPrimaryAsset(AssetID);
			}
		}
	}
}

TMap<int32, FAssetManagerChunkInfo> UAssetManager::BuildChunkMap(const TSet<FName>& PackagesToUpdateChunksFor) const
{
	return BuildChunkMap(PackagesToUpdateChunksFor, {}, {});
}

TMap<int32, FAssetManagerChunkInfo> UAssetManager::BuildChunkMap(const TSet<FName>& PackagesToUpdateChunksFor, const TSet<FName>& StartupPackages, const TSet<FName>& PackageThatWereCooked) const
{
	TMap<int32, FAssetManagerChunkInfo> ReturnChunkMap;

	TArray<int32> ChunkList;
	TArray<int32> ExistingChunkList;

	// Update management parent list, which is PrimaryAssetId -> PrimaryAssetId
	for (const TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : AssetTypeMap)
	{
		const FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();
		for (const TPair<FName, FPrimaryAssetData>& NamePair : TypeData.GetAssets())
		{
			const FPrimaryAssetData& NameData = NamePair.Value;
			FPrimaryAssetId PrimaryAssetId(TypePair.Key, NamePair.Key);
			const FSoftObjectPath& AssetRef = NameData.GetAssetPtr().ToSoftObjectPath();

			TSet<FPrimaryAssetId> Managers;
			if (AssetRef.IsValid())
			{
				FName PackageName = FName(*AssetRef.GetLongPackageName());
				TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty> ManagersMap;
				GetPackageManagers(PackageName, false, ManagersMap);
				ManagersMap.GetKeys(Managers);
			}
			else
			{
				Managers.Add(PrimaryAssetId);
			}

			// Compute chunk assignment and store those as manager references
			ChunkList.Reset();
			GetPrimaryAssetSetChunkIds(Managers, nullptr, ExistingChunkList, ChunkList);

			for (int32 ChunkId : ChunkList)
			{
				FPrimaryAssetId ChunkPrimaryAsset = CreatePrimaryAssetIdFromChunkId(ChunkId);

				ReturnChunkMap.FindOrAdd(ChunkId).ExplicitAssets.Add(PrimaryAssetId);
			}
		}
	}

	// Update the editor preview chunk package list for all chunks, but only if we actually care about chunks
	// bGenerateChunks is settable per platform, but should be enabled on the default platform for preview to work
	TArray<int32> OverrideChunkList;
	for (FName PackageName : PackagesToUpdateChunksFor)
	{
		ChunkList.Reset();
		OverrideChunkList.Reset();
		GetPackageChunkIds(PackageName, nullptr, ExistingChunkList, ChunkList, &OverrideChunkList);

		if (ChunkList.Num() > 0)
		{
			for (int32 ChunkId : ChunkList)
			{
				ReturnChunkMap.FindOrAdd(ChunkId).AllAssets.Add(PackageName);

				if (OverrideChunkList.Contains(ChunkId))
				{
					// This was in the override list, so add an explicit dependency
					ReturnChunkMap.FindOrAdd(ChunkId).ExplicitAssets.Add(PackageName);
				}
			}
		}
	}

	return ReturnChunkMap;
}

void UAssetManager::ReinitializeFromConfig()
{
	// We specifically do not reset AssetRuleOverrides as those can be set by something other than inis
	for (TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& Pair : AssetTypeMap)
	{
		if (!Pair.Value->Info.bIsDynamicAsset)
		{
			Pair.Value->ResetAssets(AssetPathMap);
		}
	}
	check(AssetPathMap.IsEmpty()); // Should have been emptied by the ResetAssets calls
	CookRuleManagementParentMap.Reset();
	ManagementParentMap.Reset();
	CachedAssetBundles.Reset();
	AlreadyScannedDirectories.Reset();

	TMap<FName, TSharedRef<FPrimaryAssetTypeData>> OldAssetTypeMap = MoveTemp(AssetTypeMap);
	AssetTypeMap.Reset();

	// This code is editor only, so reinitialize globals
	const UAssetManagerSettings& Settings = GetSettings();
	bShouldGuessTypeAndName = Settings.bShouldGuessTypeAndNameInEditor;
	bShouldAcquireMissingChunksOnLoad = Settings.bShouldAcquireMissingChunksOnLoad;
	bOnlyCookProductionAssets = Settings.bOnlyCookProductionAssets;

	if (FCoreUObjectDelegates::GetPrimaryAssetIdForObject.IsBoundToObject(this))
	{
		FCoreUObjectDelegates::GetPrimaryAssetIdForObject.Unbind();
	}
	if (Settings.bShouldManagerDetermineTypeAndName)
	{
		FCoreUObjectDelegates::GetPrimaryAssetIdForObject.BindUObject(this, &UAssetManager::DeterminePrimaryAssetIdForObject);
	}

	LoadRedirectorMaps();
	ScanPrimaryAssetTypesFromConfig();

	// Go through old list and restore data that was added after the initial config load
	for (TPair<FName, TSharedRef<FPrimaryAssetTypeData>>& TypePair : OldAssetTypeMap)
	{
		FPrimaryAssetTypeData& TypeData = TypePair.Value.Get();

		if (TypeData.Info.bIsDynamicAsset)
		{
			// Restore dynamic assets as they were before
			AssetTypeMap.Add(TypePair.Key, TypePair.Value);
		}
		else if (TypeData.AdditionalAssetScanPaths.Num())
		{
			// Rescan any paths added after initial scan
			ScanPathsForPrimaryAssets(TypePair.Key, TypeData.AdditionalAssetScanPaths.Array(), TypeData.Info.AssetBaseClassLoaded, TypeData.Info.bHasBlueprintClasses, TypeData.Info.bIsEditorOnly, false);
		}
	}
}

void UAssetManager::OnInMemoryAssetCreated(UObject *Object)
{
	// Ignore PIE and CDO changes
	if (GIsPlayInEditorWorld || !Object || Object->HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	FPrimaryAssetId PrimaryAssetId = Object->GetPrimaryAssetId();

	if (PrimaryAssetId.IsValid())
	{
		TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetId.PrimaryAssetType);

		if (FoundType)
		{
			IAssetRegistry& AssetRegistry = GetAssetRegistry();

			FPrimaryAssetTypeData& TypeData = FoundType->Get();

			FAssetData NewAssetData;

			GetAssetDataForPathInternal(AssetRegistry, Object->GetPathName(), NewAssetData);

			if (NewAssetData.IsValid() && NewAssetData.IsTopLevelAsset())
			{
				// Make sure it's in a valid path
				bool bFoundPath = false;
				for (const FString& Path : TypeData.RealAssetScanPaths)
				{
					if (NewAssetData.PackageName.ToString().StartsWith(Path))
					{
						bFoundPath = true;
						break;
					}
				}

				if (bFoundPath)
				{
					// Add or update asset data
					if (TryUpdateCachedAssetData(PrimaryAssetId, NewAssetData, true))
					{
						OnObjectReferenceListInvalidated();
					}
				}
			}
		}
	}
}

void UAssetManager::OnInMemoryAssetDeleted(UObject *Object)
{
	// Ignore PIE changes
	if (GIsPlayInEditorWorld || !Object)
	{
		return;
	}

	FPrimaryAssetId PrimaryAssetId = Object->GetPrimaryAssetId();

	RemovePrimaryAssetId(PrimaryAssetId);
}

void UAssetManager::OnObjectPreSave(UObject* Object, FObjectPreSaveContext SaveContext)
{
	// If this is in the asset manager dictionary, make sure it actually has a primary asset id that matches
	const bool bIsAssetOrClass = Object->IsAsset() || Object->IsA(UClass::StaticClass()); 
	if (!bIsAssetOrClass)
	{
		return;
	}

	FPrimaryAssetId FoundPrimaryAssetId = GetPrimaryAssetIdForPath(FSoftObjectPath(Object));
	if (FoundPrimaryAssetId.IsValid())
	{
		TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(FoundPrimaryAssetId.PrimaryAssetType);
		FPrimaryAssetId ObjectPrimaryAssetId = Object->GetPrimaryAssetId();

		if (FoundPrimaryAssetId != ObjectPrimaryAssetId && !(*FoundType)->Info.bIsEditorOnly)
		{
			UE_LOGF(LogAssetManager, Error, "Registered PrimaryAssetId %ls for asset %ls does not match object's real id of %ls! This will not load properly at runtime!", *FoundPrimaryAssetId.ToString(), *Object->GetPathName(), *ObjectPrimaryAssetId.ToString());
		}
	}
}

void UAssetManager::OnAssetRenamed(const FAssetData& NewData, const FString& OldPath)
{
	// Ignore PIE changes
	if (GIsPlayInEditorWorld || !NewData.IsValid())
	{
		return;
	}

	FPrimaryAssetId OldPrimaryAssetId = GetPrimaryAssetIdForPath(OldPath);

	// This may be a blueprint, try with _C
	if (!OldPrimaryAssetId.IsValid())
	{
		OldPrimaryAssetId = GetPrimaryAssetIdForPath(OldPath + TEXT("_C"));
	}

	RemovePrimaryAssetId(OldPrimaryAssetId);

	// This will always be in memory
	UObject *NewObject = NewData.GetAsset();

	OnInMemoryAssetCreated(NewObject);
}

void UAssetManager::OnAssetRemoved(const FAssetData& Data)
{
	// This could be much more efficient if UAssetManager broadcast one large event instead of all these tiny updates, see UAssetRegistryImpl::Broadcast

	FPrimaryAssetId PrimaryAssetId = GetPrimaryAssetIdForPath(Data.GetSoftObjectPath());

	// This may be a blueprint, try with _C
	if (!PrimaryAssetId.IsValid())
	{
		FSoftObjectPath Path(WriteToString<FName::StringBufferSize>(Data.GetSoftObjectPath(), TEXT("_C")));
		PrimaryAssetId = GetPrimaryAssetIdForPath(Path);
	}

	CachedAssetBundles.Remove(PrimaryAssetId);

	RemovePrimaryAssetId(PrimaryAssetId);
}

void UAssetManager::RemovePrimaryAssetId(const FPrimaryAssetId& PrimaryAssetId)
{
	if (PrimaryAssetId.IsValid() && GetNameData(PrimaryAssetId))
	{
		// It's in our dictionary, remove it

		TSharedRef<FPrimaryAssetTypeData>* FoundType = AssetTypeMap.Find(PrimaryAssetId.PrimaryAssetType);
		check(FoundType);
		FPrimaryAssetTypeData& TypeData = FoundType->Get();

		TypeData.RemoveAsset(PrimaryAssetId.PrimaryAssetName, AssetPathMap);

		OnObjectReferenceListInvalidated();
	}
}

void UAssetManager::RefreshAssetData(UObject* ChangedObject)
{
	// If this is a BP CDO, call on class instead
	if (ChangedObject->HasAnyFlags(RF_ClassDefaultObject))
	{
		UBlueprintGeneratedClass* AssetClass = Cast<UBlueprintGeneratedClass>(ChangedObject->GetClass());
		if (AssetClass)
		{
			RefreshAssetData(AssetClass);
		}
		return;
	}

	// Only update things it knows about
	IAssetRegistry& AssetRegistry = GetAssetRegistry();
	FSoftObjectPath ChangedObjectPath(ChangedObject);
	FPrimaryAssetId PrimaryAssetId = ChangedObject->GetPrimaryAssetId();
	FPrimaryAssetId OldPrimaryAssetId = GetPrimaryAssetIdForPath(ChangedObjectPath);
	
	// This may be a blueprint, try with _C
	if (!OldPrimaryAssetId.IsValid())
	{
		OldPrimaryAssetId = GetPrimaryAssetIdForPath(ChangedObjectPath.ToString() + TEXT("_C"));
	}

	if (PrimaryAssetId.IsValid() && OldPrimaryAssetId == PrimaryAssetId)
	{
		// Same AssetId, this will update cache out of the in memory object
		UClass* Class = Cast<UClass>(ChangedObject);
		FAssetData NewData(Class && Class->ClassGeneratedBy ? ToRawPtr(Class->ClassGeneratedBy) : ToRawPtr(ChangedObject));

		if (ensure(NewData.IsValid()))
		{
			TryUpdateCachedAssetData(PrimaryAssetId, NewData, false);
		}
	}
	else
	{
		// AssetId changed
		if (OldPrimaryAssetId.IsValid())
		{
			// Remove old id if it was registered
			RemovePrimaryAssetId(OldPrimaryAssetId);
		}

		if (PrimaryAssetId.IsValid())
		{
			// This will add new id
			OnInMemoryAssetCreated(ChangedObject);
		}
	}
}

void UAssetManager::InitializeAssetBundlesFromMetadata(const UStruct* Struct, const void* StructValue, FAssetBundleData& AssetBundle, FName DebugName) const
{
	TSet<const void*> AllVisitedStructValues;
	InitializeAssetBundlesFromMetadata_Recursive(Struct, StructValue, AssetBundle, DebugName, AllVisitedStructValues);
}

void UAssetManager::InitializeAssetBundlesFromMetadata_Recursive(const UStruct* Struct, const void* StructValue, FAssetBundleData& AssetBundle, FName DebugName, TSet<const void*>& AllVisitedStructValues) const
{
	static FName AssetBundlesName = "AssetBundles";
	static FName IncludeAssetBundlesName = "IncludeAssetBundles";

	if (!ensure(Struct && StructValue))
	{
		return;
	}

	if (AllVisitedStructValues.Contains(StructValue))
	{
		return;
	}

	AllVisitedStructValues.Add(StructValue);

	for (TPropertyValueIterator<const FProperty> It(Struct, StructValue); It; ++It)
	{
		const FProperty* Property = It.Key();
		const void* PropertyValue = It.Value();

		FSoftObjectPath FoundRef;
		if (const FSoftClassProperty* AssetClassProp = CastField<FSoftClassProperty>(Property))
		{
			const FSoftObjectPtr& AssetClassPtr = AssetClassProp->GetPropertyValue(PropertyValue);
			FoundRef = AssetClassPtr.ToSoftObjectPath();
		}
		else if (const FSoftObjectProperty* AssetProp = CastField<FSoftObjectProperty>(Property))
		{
			const FSoftObjectPtr& AssetClassPtr = AssetProp->GetPropertyValue(PropertyValue);
			FoundRef = AssetClassPtr.ToSoftObjectPath();
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			// SoftClassPath is binary identical with SoftObjectPath
			if (StructProperty->Struct == TBaseStructure<FSoftObjectPath>::Get() || StructProperty->Struct == TBaseStructure<FSoftClassPath>::Get())
			{
				const FSoftObjectPath* AssetRefPtr = reinterpret_cast<const FSoftObjectPath*>(PropertyValue);
				if (AssetRefPtr)
				{
					FoundRef = *AssetRefPtr;
				}
				// Skip recursion, we don't care about the raw string property
				It.SkipRecursiveProperty();
			}
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			if (ObjectProperty->PropertyFlags & CPF_InstancedReference || ObjectProperty->GetOwnerProperty()->HasMetaData(IncludeAssetBundlesName))
			{
				const UObject* Object = ObjectProperty->GetObjectPropertyValue(PropertyValue);
				if (Object != nullptr)
				{
					InitializeAssetBundlesFromMetadata_Recursive(Object->GetClass(), Object, AssetBundle, Object->GetFName(), AllVisitedStructValues);
				}
			}
		}

		if (!FoundRef.IsNull())
		{
			if (!FoundRef.GetLongPackageName().IsEmpty())
			{
				// Compute the intersection of all specified bundle sets in this property and parent properties
				TSet<FName> BundleSet;

				TArray<const FProperty*> PropertyChain;
				It.GetPropertyChain(PropertyChain);

				for (const FProperty* PropertyToSearch : PropertyChain)
				{
					if (PropertyToSearch->HasMetaData(AssetBundlesName))
					{
						TSet<FName> LocalBundleSet;
						TArray<FString> BundleList;
						const FString& BundleString = PropertyToSearch->GetMetaData(AssetBundlesName);
						BundleString.ParseIntoArrayWS(BundleList, TEXT(","));

						for (const FString& BundleNameString : BundleList)
						{
							LocalBundleSet.Add(FName(*BundleNameString));
						}

						// If Set is empty, initialize. Otherwise intersect
						if (BundleSet.Num() == 0)
						{
							BundleSet = LocalBundleSet;
						}
						else
						{
							BundleSet = BundleSet.Intersect(LocalBundleSet);
						}
					}
				}

				for (const FName& BundleName : BundleSet)
				{
					AssetBundle.AddBundleAsset(BundleName, FoundRef.GetAssetPath());
				}
			}
			else
			{
				UE_LOGF(LogAssetManager, Error, "Asset bundle reference with invalid package name in %ls. Property:%ls", *DebugName.ToString(), *GetNameSafe(Property));
			}
		}
	}
}

#endif // #if WITH_EDITOR

#if !UE_BUILD_SHIPPING

// Cheat command to load all assets of a given type
static FAutoConsoleCommandWithWorldAndArgs CVarLoadPrimaryAssetsWithType(
	TEXT("AssetManager.LoadPrimaryAssetsWithType"),
	TEXT("Loads all assets of a given type"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
		[](const TArray<FString>& Params, UWorld* World)
{
	if (Params.Num() == 0)
	{
		UE_LOGF(LogAssetManager, Log, "No types specified");
	}

	for (const FString& Param : Params)
	{
		const FPrimaryAssetType TypeToLoad(*Param);

		FPrimaryAssetTypeInfo Info;
		if (UAssetManager::Get().GetPrimaryAssetTypeInfo(TypeToLoad, /*out*/ Info))
		{
			UE_LOGF(LogAssetManager, Log, "LoadPrimaryAssetsWithType(%ls)", *Param);
			UAssetManager::Get().LoadPrimaryAssetsWithType(TypeToLoad);
		}
		else
		{
			UE_LOGF(LogAssetManager, Log, "Cannot get type info for PrimaryAssetType %ls", *Param);
		}		
	}
}), ECVF_Cheat);

// Cheat command to unload all assets of a given type
static FAutoConsoleCommandWithWorldAndArgs CVarUnloadPrimaryAssetsWithType(
	TEXT("AssetManager.UnloadPrimaryAssetsWithType"),
	TEXT("Unloads all assets of a given type"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
		[](const TArray<FString>& Params, UWorld* World)
{
	if (Params.Num() == 0)
	{
		UE_LOGF(LogAssetManager, Log, "No types specified");
	}

	for (const FString& Param : Params)
	{
		const FPrimaryAssetType TypeToUnload(*Param);

		FPrimaryAssetTypeInfo Info;
		if (UAssetManager::Get().GetPrimaryAssetTypeInfo(TypeToUnload, /*out*/ Info))
		{
			int32 NumUnloaded = UAssetManager::Get().UnloadPrimaryAssetsWithType(TypeToUnload);
			UE_LOGF(LogAssetManager, Log, "UnloadPrimaryAssetsWithType(%ls): Unloaded %d assets", *Param, NumUnloaded);
		}
		else
		{
			UE_LOGF(LogAssetManager, Log, "Cannot get type info for PrimaryAssetType %ls", *Param);
		}
	}
}), ECVF_Cheat);

#endif

#undef LOCTEXT_NAMESPACE

