// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/AssetManagerTypes.h"
#include "Engine/AssetManager.h"
#include "Engine/AssetManagerSettings.h"
#include "Engine/DeveloperSettings.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetManagerTypes)

bool FPrimaryAssetTypeInfo::HasValidConfigData() const
{
	if (PrimaryAssetType == NAME_None)
	{
		return false;
	}

	if (AssetBaseClass.IsNull())
	{
		return false;
	}

	// No paths are required

	return true;
}

bool FPrimaryAssetTypeInfo::CanModifyConfigData() const
{
	// Can't modify config data after adding paths
	return AssetScanPaths.Num() == 0;
}

bool FPrimaryAssetTypeInfo::HasValidRuntimeData() const
{
	if (PrimaryAssetType == NAME_None)
	{
		return false;
	}

	if (AssetBaseClassLoaded == nullptr)
	{
		return false;
	}

	// Invalid if the paths haven't been copied over yet, this is valid if all paths are empty
	if ((AssetScanPaths.Num() == 0) && (Directories.Num() + SpecificAssets.Num() > 0))
	{
		return false;
	}

	return true;
}

void FPrimaryAssetTypeInfo::FillRuntimeData(bool& bIsValid, bool& bBaseClassWasLoaded)
{
	bBaseClassWasLoaded = false;
	bIsValid = false;

	if (PrimaryAssetType == NAME_None)
	{
		// Invalid type
		return;
	}

	if (!ensureMsgf(!AssetBaseClass.IsNull(), TEXT("Primary Asset Type %s must have a class set!"), *PrimaryAssetType.ToString()))
	{
		return;
	}

	// Hot reload may have messed up asset pointer
	AssetBaseClass.ResetWeakPtr();
	AssetBaseClassLoaded = AssetBaseClass.Get();

	if (!AssetBaseClassLoaded)
	{
		bBaseClassWasLoaded = true;
		AssetBaseClassLoaded = AssetBaseClass.LoadSynchronous();
	}

	if (!ensureMsgf(AssetBaseClassLoaded, TEXT("Failed to load class %s for Primary Asset Type %s!"), *AssetBaseClass.ToString(), *PrimaryAssetType.ToString()))
	{
		bBaseClassWasLoaded = false;
		return;
	}

	for (const FSoftObjectPath& AssetRef : SpecificAssets)
	{
		if (!AssetRef.IsNull())
		{
			AssetScanPaths.AddUnique(AssetRef.ToString());
		}
	}

	for (const FDirectoryPath& PathRef : Directories)
	{
		if (!PathRef.Path.IsEmpty())
		{
			AssetScanPaths.AddUnique(UAssetManager::GetNormalizedPackagePath(PathRef.Path, false));
		}
	}

	// Valid data, it's fine for a type to have no scan directories
	bIsValid = ensureMsgf(HasValidRuntimeData(), TEXT("Failed to FillRuntimeData for Primary Asset Type %s"), *PrimaryAssetType.ToString());
}

bool FPrimaryAssetRules::IsDefault() const
{
	return *this == FPrimaryAssetRules();
}

void FPrimaryAssetRules::OverrideRules(const FPrimaryAssetRules& OverrideRules)
{
	static FPrimaryAssetRules DefaultRules;

	if (OverrideRules.Priority != DefaultRules.Priority)
	{
		Priority = OverrideRules.Priority;
	}

	if (OverrideRules.bApplyRecursively != DefaultRules.bApplyRecursively)
	{
		bApplyRecursively = OverrideRules.bApplyRecursively;
	}

	if (OverrideRules.ChunkId != DefaultRules.ChunkId)
	{
		ChunkId = OverrideRules.ChunkId;
	}

	if (OverrideRules.CookRule != DefaultRules.CookRule)
	{
		CookRule = OverrideRules.CookRule;
	}
}

void FPrimaryAssetRulesExplicitOverride::OverrideRulesExplicitly(FPrimaryAssetRules& RulesToOverride) const
{
	if (bOverridePriority)
	{
		RulesToOverride.Priority = Rules.Priority;
	}

	if (bOverrideApplyRecursively)
	{
		RulesToOverride.bApplyRecursively = Rules.bApplyRecursively;
	}

	if (bOverrideChunkId)
	{
		RulesToOverride.ChunkId = Rules.ChunkId;
	}

	if (bOverrideCookRule)
	{
		RulesToOverride.CookRule = Rules.CookRule;
	}
}

void FPrimaryAssetRules::PropagateCookRules(const FPrimaryAssetRules& ParentRules)
{
	static FPrimaryAssetRules DefaultRules;

	if (ParentRules.ChunkId != DefaultRules.ChunkId && ChunkId == DefaultRules.ChunkId)
	{
		ChunkId = ParentRules.ChunkId;
	}

	if (ParentRules.CookRule != DefaultRules.CookRule && CookRule == DefaultRules.CookRule)
	{
		CookRule = ParentRules.CookRule;
	}
}

bool FAssetManagerSearchRules::AreRulesSet() const
{	
	if (AssetScanPaths.Num() || ExcludePatterns.Num() || IncludePatterns.Num())
	{
		return true;
	}
	else if (AssetBaseClass)
	{
		return true;
	}
	else if (ShouldIncludeDelegate.IsBound())
	{
		return true;
	}

	return false;
}

void UAssetManagerSettings::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

	UAssetManager::Get().LoadRedirectorMaps();
}

void SerializeForLog(FCbWriter& Writer, const FPrimaryAssetRules& Value)
{
	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("PrimaryAssetRules"));
	// no text field
	Writer.AddInteger(ANSITEXTVIEW("Priority"), Value.Priority);
	Writer.AddInteger(ANSITEXTVIEW("ChunkId"), Value.ChunkId);
	Writer.AddBool(ANSITEXTVIEW("Recursive"), Value.bApplyRecursively);
	Writer.AddString(ANSITEXTVIEW("CookRule"), UEnum::GetValueAsString(Value.CookRule));
	Writer.EndObject();
}

#if WITH_EDITOR
void UAssetManagerSettings::PostInitProperties()
{
	Super::PostInitProperties();

	ApplyMetaDataTagsSettings();
}

void UAssetManagerSettings::ApplyMetaDataTagsSettings()
{
	TSet<FName>& GlobalTagsForAssetRegistry = UObject::GetMetaDataTagsForAssetRegistry();
	for (FName Tag : MetaDataTagsForAssetRegistry)
	{
		if (!Tag.IsNone())
		{
			if (!GlobalTagsForAssetRegistry.Contains(Tag))
			{
				GlobalTagsForAssetRegistry.Add(Tag);
			}
			else
			{
				// To catch the case where the same tag is used by different users and their settings are synced after edition
				UE_LOGF(LogAssetManager, Warning, "Cannot use duplicate metadata tag '%ls' for Asset Registry", *Tag.ToString());
			}
		}
	}
}

void UAssetManagerSettings::ClearMetaDataTagsSettings()
{
	TSet<FName>& GlobalTagsForAssetRegistry = UObject::GetMetaDataTagsForAssetRegistry();
	for (FName Tag : MetaDataTagsForAssetRegistry)
	{
		if (!Tag.IsNone())
		{
			GlobalTagsForAssetRegistry.Remove(Tag);
		}
	}
}

void UAssetManagerSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange != nullptr)
	{
		FName PropertyName = PropertyAboutToChange->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UAssetManagerSettings, MetaDataTagsForAssetRegistry))
		{
			ClearMetaDataTagsSettings();
		}
	}
}

void UAssetManagerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAssetManagerSettings, MetaDataTagsForAssetRegistry))
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
		{
			// Check if the new value already exists in the global tags list
			int32 Index = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
			if (Index > 0)
			{
				TSet<FName>::TIterator It = MetaDataTagsForAssetRegistry.CreateIterator();
				for (int32 i = 0; i < Index; ++i)
				{
					++It;
				}
				FName NewValue = It ? *It : FName();
				if (UObject::GetMetaDataTagsForAssetRegistry().Contains(NewValue))
				{
					*It = FName();
					UE_LOGF(LogAssetManager, Warning, "Cannot use duplicate metadata tag '%ls' for Asset Registry", *NewValue.ToString());
				}
			}
		}
		ApplyMetaDataTagsSettings();
	}
	else if (PropertyChangedEvent.Property && UAssetManager::IsInitialized())
	{
		UAssetManager::Get().ReinitializeFromConfig();
	}
}
#endif
