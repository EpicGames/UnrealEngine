// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/AssetUserData.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "UObject/Interface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetUserData)


UAssetUserData::UAssetUserData(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
{

}

//////////////////////////////////////////////////////////////////////////

UInterface_AssetUserData::UInterface_AssetUserData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

