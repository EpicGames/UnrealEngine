// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncActionLoadPrimaryAsset.h"
#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncActionLoadPrimaryAsset)

void UAsyncActionLoadPrimaryAssetBase::Activate()
{
	check(UAssetManager::IsInitialized());
	UAssetManager& Manager = UAssetManager::Get();
	switch (Operation)
	{
	case EAssetManagerOperation::Load:
		LoadHandle = Manager.LoadPrimaryAssets(AssetsToLoad, LoadBundles);
		break;
	case EAssetManagerOperation::ChangeBundleStateMatching:
		LoadHandle = Manager.ChangeBundleStateForMatchingPrimaryAssets(LoadBundles, OldBundles);
		break;
	case EAssetManagerOperation::ChangeBundleStateList:
		LoadHandle = Manager.ChangeBundleStateForPrimaryAssets(AssetsToLoad, LoadBundles, OldBundles);
		break;
	}
		
	if (LoadHandle.IsValid())
	{
		if (!LoadHandle->HasLoadCompleted())
		{
			LoadHandle->BindCompleteDelegate(FStreamableDelegate::CreateUObject(this, &UAsyncActionLoadPrimaryAssetBase::HandleLoadCompleted));
			return;
		}
	}

	// Either load already succeeded, or it failed
	HandleLoadCompleted();
}

void UAsyncActionLoadPrimaryAssetBase::HandleLoadCompleted()
{
	LoadHandle.Reset();
	SetReadyToDestroy();
}

void UAsyncActionLoadPrimaryAssetBase::GetCurrentlyLoadedAssets(TArray<UObject*>& AssetList)
{
	check(UAssetManager::IsInitialized());
	UAssetManager& Manager = UAssetManager::Get();
	// The assets may have already been loaded but the handle was invalid, check the original list
	for (const FPrimaryAssetId& IdToLoad : AssetsToLoad)
	{
		UObject* LoadedObject = Manager.GetPrimaryAssetObject(IdToLoad);
		if (LoadedObject)
		{
			AssetList.Add(LoadedObject);
		}
	}
}

UAsyncActionLoadPrimaryAsset* UAsyncActionLoadPrimaryAsset::AsyncLoadPrimaryAsset(UObject* WorldContextObject, FPrimaryAssetId PrimaryAsset, const TArray<FName>& LoadBundles)
{
	UAsyncActionLoadPrimaryAsset* Action = NewObject<UAsyncActionLoadPrimaryAsset>();
	Action->AssetsToLoad.Add(PrimaryAsset);
	Action->LoadBundles = LoadBundles;
	Action->Operation = EAssetManagerOperation::Load;
	Action->RegisterWithGameInstance(WorldContextObject);

	return Action;
}

void UAsyncActionLoadPrimaryAsset::HandleLoadCompleted()
{
	UObject* AssetLoaded = nullptr;
	TArray<UObject*> AssetList;

	GetCurrentlyLoadedAssets(AssetList);

	if (AssetList.Num() > 0)
	{
		AssetLoaded = AssetList[0];
	}

	Super::HandleLoadCompleted();
	Completed.Broadcast(AssetLoaded);
}

UAsyncActionLoadPrimaryAssetClass* UAsyncActionLoadPrimaryAssetClass::AsyncLoadPrimaryAssetClass(UObject* WorldContextObject, FPrimaryAssetId PrimaryAsset, const TArray<FName>& LoadBundles)
{
	UAsyncActionLoadPrimaryAssetClass* Action = NewObject<UAsyncActionLoadPrimaryAssetClass>();
	Action->AssetsToLoad.Add(PrimaryAsset);
	Action->LoadBundles = LoadBundles;
	Action->Operation = EAssetManagerOperation::Load;
	Action->RegisterWithGameInstance(WorldContextObject);

	return Action;
}

void UAsyncActionLoadPrimaryAssetClass::HandleLoadCompleted()
{
	TSubclassOf<UObject> AssetLoaded = nullptr;
	TArray<UObject*> AssetList;

	GetCurrentlyLoadedAssets(AssetList);

	if (AssetList.Num() > 0)
	{
		AssetLoaded = Cast<UClass>(AssetList[0]);
	}

	Super::HandleLoadCompleted();
	Completed.Broadcast(AssetLoaded);
}

UAsyncActionLoadPrimaryAssetList* UAsyncActionLoadPrimaryAssetList::AsyncLoadPrimaryAssetList(UObject* WorldContextObject, const TArray<FPrimaryAssetId>& PrimaryAssetList, const TArray<FName>& LoadBundles)
{
	UAsyncActionLoadPrimaryAssetList* Action = NewObject<UAsyncActionLoadPrimaryAssetList>();
	Action->AssetsToLoad = PrimaryAssetList;
	Action->LoadBundles = LoadBundles;
	Action->Operation = EAssetManagerOperation::Load;
	Action->RegisterWithGameInstance(WorldContextObject);

	return Action;
}

void UAsyncActionLoadPrimaryAssetList::HandleLoadCompleted()
{
	TArray<UObject*> AssetList;

	GetCurrentlyLoadedAssets(AssetList);

	Super::HandleLoadCompleted();
	Completed.Broadcast(AssetList);
}

UAsyncActionLoadPrimaryAssetClassList* UAsyncActionLoadPrimaryAssetClassList::AsyncLoadPrimaryAssetClassList(UObject* WorldContextObject, const TArray<FPrimaryAssetId>& PrimaryAssetList, const TArray<FName>& LoadBundles)
{
	UAsyncActionLoadPrimaryAssetClassList* Action = NewObject<UAsyncActionLoadPrimaryAssetClassList>();
	Action->AssetsToLoad = PrimaryAssetList;
	Action->LoadBundles = LoadBundles;
	Action->Operation = EAssetManagerOperation::Load;
	Action->RegisterWithGameInstance(WorldContextObject);

	return Action;
}

void UAsyncActionLoadPrimaryAssetClassList::HandleLoadCompleted()
{
	TArray<TSubclassOf<UObject>> AssetClassList;
	TArray<UObject*> AssetList;

	GetCurrentlyLoadedAssets(AssetList);

	for (UObject* LoadedAsset : AssetList)
	{
		UClass* LoadedClass = Cast<UClass>(LoadedAsset);

		if (LoadedClass)
		{
			AssetClassList.Add(LoadedClass);
		}
	}

	Super::HandleLoadCompleted();
	Completed.Broadcast(AssetClassList);
}

UAsyncActionChangePrimaryAssetBundles* UAsyncActionChangePrimaryAssetBundles::AsyncChangeBundleStateForMatchingPrimaryAssets(UObject* WorldContextObject, const TArray<FName>& NewBundles, const TArray<FName>& OldBundles)
{
	UAsyncActionChangePrimaryAssetBundles* Action = NewObject<UAsyncActionChangePrimaryAssetBundles>();
	Action->LoadBundles = NewBundles;
	Action->OldBundles = OldBundles;
	Action->Operation = EAssetManagerOperation::ChangeBundleStateMatching;
	Action->RegisterWithGameInstance(WorldContextObject);
	
	return Action;
}

UAsyncActionChangePrimaryAssetBundles* UAsyncActionChangePrimaryAssetBundles::AsyncChangeBundleStateForPrimaryAssetList(UObject* WorldContextObject, const TArray<FPrimaryAssetId>& PrimaryAssetList, const TArray<FName>& AddBundles, const TArray<FName>& RemoveBundles)
{
	UAsyncActionChangePrimaryAssetBundles* Action = NewObject<UAsyncActionChangePrimaryAssetBundles>();
	Action->LoadBundles = AddBundles;
	Action->OldBundles = RemoveBundles;
	Action->AssetsToLoad = PrimaryAssetList;
	Action->Operation = EAssetManagerOperation::ChangeBundleStateList;
	Action->RegisterWithGameInstance(WorldContextObject);

	return Action;
}

void UAsyncActionChangePrimaryAssetBundles::HandleLoadCompleted()
{
	Super::HandleLoadCompleted();
	Completed.Broadcast();
}
