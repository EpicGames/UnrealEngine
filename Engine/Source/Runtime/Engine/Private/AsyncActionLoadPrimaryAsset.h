// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/StreamableManager.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Templates/SubclassOf.h"

#include "AsyncActionLoadPrimaryAsset.generated.h"

/** Base class of all asset manager load calls */
UCLASS(Abstract)
class UAsyncActionLoadPrimaryAssetBase : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:
	/** Execute the actual load */
	virtual void Activate() override;

protected:

	/** Called from asset manager */
	virtual void HandleLoadCompleted();

	/** Gets all UObjects matching AssetsToLoad that are already in memory */
	virtual void GetCurrentlyLoadedAssets(TArray<UObject*>& AssetList);

	/** Specific assets requested */
	TArray<FPrimaryAssetId> AssetsToLoad;

	/** Bundle states */
	TArray<FName> LoadBundles;

	/** Bundle states */
	TArray<FName> OldBundles;

	/** Handle of load request */
	TSharedPtr<FStreamableHandle> LoadHandle;

	enum class EAssetManagerOperation : uint8
	{
		Load,
		ChangeBundleStateMatching,
		ChangeBundleStateList
	};

	/** Which operation is being run */
	EAssetManagerOperation Operation;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPrimaryAssetLoaded, UObject*, Loaded);

UCLASS()
class UAsyncActionLoadPrimaryAsset : public UAsyncActionLoadPrimaryAssetBase
{
	GENERATED_BODY()

public:
	/** 
	 * Load a primary asset object into memory, this will cause it to stay loaded until it is explicitly unloaded.
	 * The completed event will happen when the load succeeds or fails, you should cast the Loaded object to verify it is the correct type.
	 * If LoadBundles is specified, those bundles are loaded along with the asset.
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true", Category = "AssetManager", AutoCreateRefTerm = "LoadBundles", WorldContext = "WorldContextObject"))
	static UAsyncActionLoadPrimaryAsset* AsyncLoadPrimaryAsset(UObject* WorldContextObject, FPrimaryAssetId PrimaryAsset, const TArray<FName>& LoadBundles);

	UPROPERTY(BlueprintAssignable)
	FOnPrimaryAssetLoaded Completed;

protected:

	/** Called from asset manager */
	virtual void HandleLoadCompleted() override;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPrimaryAssetClassLoaded, TSubclassOf<UObject>, Loaded);

UCLASS()
class UAsyncActionLoadPrimaryAssetClass : public UAsyncActionLoadPrimaryAssetBase
{
	GENERATED_BODY()

public:
	/**
	 * Load a primary asset class  into memory, this will cause it to stay loaded until it is explicitly unloaded.
	 * The completed event will happen when the load succeeds or fails, you should cast the Loaded class to verify it is the correct type.
	 * If LoadBundles is specified, those bundles are loaded along with the asset.
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", Category = "AssetManager", AutoCreateRefTerm = "LoadBundles", WorldContext = "WorldContextObject"))
	static UAsyncActionLoadPrimaryAssetClass* AsyncLoadPrimaryAssetClass(UObject* WorldContextObject, FPrimaryAssetId PrimaryAsset, const TArray<FName>& LoadBundles);

	UPROPERTY(BlueprintAssignable)
	FOnPrimaryAssetClassLoaded Completed;

protected:

	/** Called from asset manager */
	virtual void HandleLoadCompleted() override;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPrimaryAssetListLoaded, const TArray<UObject*>&, Loaded);

UCLASS()
class UAsyncActionLoadPrimaryAssetList : public UAsyncActionLoadPrimaryAssetBase
{
	GENERATED_BODY()

public:
	/**
	 * Load a list of primary asset objects into memory, this will cause them to stay loaded until explicitly unloaded.
	 * The completed event will happen when the load succeeds or fails, and the Loaded list will contain all of the requested assets found at completion.
	 * If LoadBundles is specified, those bundles are loaded along with the assets.
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true", Category = "AssetManager", AutoCreateRefTerm = "LoadBundles", WorldContext = "WorldContextObject"))
	static UAsyncActionLoadPrimaryAssetList* AsyncLoadPrimaryAssetList(UObject* WorldContextObject, const TArray<FPrimaryAssetId>& PrimaryAssetList, const TArray<FName>& LoadBundles);

	UPROPERTY(BlueprintAssignable)
	FOnPrimaryAssetListLoaded Completed;

protected:

	/** Called from asset manager */
	virtual void HandleLoadCompleted() override;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPrimaryAssetClassListLoaded, const TArray<TSubclassOf<UObject>>&, Loaded);

UCLASS()
class UAsyncActionLoadPrimaryAssetClassList : public UAsyncActionLoadPrimaryAssetBase
{
	GENERATED_BODY()

public:
	/**
	 * Load a list of primary asset classes into memory, this will cause them to stay loaded until explicitly unloaded.
	 * The completed event will happen when the load succeeds or fails, and the Loaded list will contain all of the requested classes found at completion.
	 * If LoadBundles is specified, those bundles are loaded along with the assets.
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", Category = "AssetManager", AutoCreateRefTerm = "LoadBundles", WorldContext = "WorldContextObject"))
	static UAsyncActionLoadPrimaryAssetClassList* AsyncLoadPrimaryAssetClassList(UObject* WorldContextObject, const TArray<FPrimaryAssetId>& PrimaryAssetList, const TArray<FName>& LoadBundles);

	UPROPERTY(BlueprintAssignable)
	FOnPrimaryAssetClassListLoaded Completed;

protected:

	/** Called from asset manager */
	virtual void HandleLoadCompleted() override;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPrimaryAssetBundlesChanged);

UCLASS()
class UAsyncActionChangePrimaryAssetBundles : public UAsyncActionLoadPrimaryAssetBase
{
	GENERATED_BODY()

public:
	/**
	 * Change the bundle state of all assets that match OldBundles to instead contain NewBundles.
	 * This will not change the loaded status of primary assets but will load or unload secondary assets based on the bundles.
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", Category = "AssetManager", WorldContext = "WorldContextObject"))
	static UAsyncActionChangePrimaryAssetBundles* AsyncChangeBundleStateForMatchingPrimaryAssets(UObject* WorldContextObject, const TArray<FName>& NewBundles, const TArray<FName>& OldBundles);

	/**
	 * Change the bundle state of specific assets in PrimaryAssetList. 
	 * AddBundles are added to the final state and RemoveBundles are removed, an empty array will make no change.
	 * This will not change the loaded status of primary assets but will load or unload secondary assets based on the bundles.
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", Category = "AssetManager", WorldContext = "WorldContextObject"))
	static UAsyncActionChangePrimaryAssetBundles* AsyncChangeBundleStateForPrimaryAssetList(UObject* WorldContextObject, const TArray<FPrimaryAssetId>& PrimaryAssetList, const TArray<FName>& AddBundles, const TArray<FName>& RemoveBundles);

	UPROPERTY(BlueprintAssignable)
	FOnPrimaryAssetBundlesChanged Completed;

protected:

	/** Called from asset manager */
	virtual void HandleLoadCompleted() override;
};
