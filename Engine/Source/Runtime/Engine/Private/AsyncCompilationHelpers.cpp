// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncCompilationHelpers.h"
#include "UObject/UnrealType.h"

#include "AssetCompilingManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/QueuedThreadPool.h"
#include "ObjectCacheContext.h"
#include "Misc/CommandLine.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "HAL/PlatformStackWalk.h"
#include "ShaderCompiler.h"
#include "ProfilingDebugging/StallDetector.h"
#include "UObject/UObjectThreadContext.h"

#if WITH_EDITOR
#include "Settings/EditorExperimentalSettings.h"
#endif

DEFINE_LOG_CATEGORY(LogAsyncCompilation);

#define LOCTEXT_NAMESPACE "AsyncCompilation"

void FAsyncCompilationNotification::Update(int32 NumJobs)
{
#if WITH_EDITOR
	check(IsInGameThread());

	// Use a lambda to only for progress message in code-path where it's needed
	auto GetProgressMessage =
		[this, NumJobs]()
	{
		FFormatNamedArguments Args;
		// Always use the plural form when displaying the notification
		Args.Add(TEXT("AssetType"), FText::Format(AssetNameFormat, FText::AsNumber(100)));
		Args.Add(TEXT("NumJobs"), FText::AsNumber(NumJobs));
		return FText::Format(LOCTEXT("AsyncCompilationProgress", "Preparing {AssetType} ({NumJobs})"), Args);
	};

	if (NumJobs == 0)
	{
		if (NotificationHandle.IsValid())
		{
			FSlateNotificationManager::Get().UpdateProgressNotification(NotificationHandle, StartNumJobs, StartNumJobs, GetProgressMessage());
		}
		StartNumJobs = 0;
		NotificationHandle = FProgressNotificationHandle();
	}
	else
	{
		if (!NotificationHandle.IsValid())
		{
			StartNumJobs = NumJobs;
			NotificationHandle = FSlateNotificationManager::Get().StartProgressNotification(GetProgressMessage(), StartNumJobs);
		}
		else
		{
			if (NumJobs > StartNumJobs)
			{
				StartNumJobs = NumJobs;
			}
			FSlateNotificationManager::Get().UpdateProgressNotification(NotificationHandle, StartNumJobs - NumJobs, StartNumJobs, GetProgressMessage());
		}
	}
#endif
};

#if WITH_EDITOR

namespace AsyncCompilationHelpers
{
	void DumpStallStacks();
}

static FAutoConsoleCommand CVarAsyncAssetDumpStallStacks(
	TEXT("Editor.AsyncAssetDumpStallStacks"),
	TEXT("Dump all the callstacks that have caused waits on async compilation."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		AsyncCompilationHelpers::DumpStallStacks();
	})
);

#endif // WITH_EDITOR

namespace AsyncCompilationHelpers
{
	void FinishCompilation(
		TFunctionRef<ICompilable& (int32 Index)> Getter,
		int32 Num,
#if WITH_EDITOR
		const FText& AssetType,
		const FLogCategoryBase& LogCategory,
#endif
		TFunctionRef<void(ICompilable*)> PostCompileSingle
	)
	{
		if (Num <= 0)
		{
			return;
		}

		check(IsInGameThread());

		FObjectCacheContextScope ObjectCacheScope;

		uint64 StartTime = FPlatformTime::Cycles64();
#if WITH_EDITOR
		TOptional<FScopedSlowTask> SlowTask;

		// Do not create a progress during PostLoad or garbage collect as ticking slate might
		// introduce too many unwanted side effects
		if (!FUObjectThreadContext::Get().IsRoutingPostLoad && !GIsGarbageCollecting)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("AssetType"), AssetType.ToLower());

			SlowTask.Emplace((float)Num, FText::Format(LOCTEXT("WaitingOnFinishCompilation", "Waiting on {AssetType} preparation"), Args), true);
			SlowTask->MakeDialogDelayed(1.0f, false /*bShowCancelButton*/, false /*bAllowInPIE*/);
		}

		auto FormatProgress =
			[&AssetType](int32 Done, int32 Total, FName ObjectName)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("AssetType"), AssetType.ToLower());
				Args.Add(TEXT("Done"), Done);
				Args.Add(TEXT("Total"), Total);
				Args.Add(TEXT("ObjectName"), FText::FromName(ObjectName));
				return FText::Format(LOCTEXT("WaitingOnFinishCompilationWithCount", "Waiting for {AssetType} to be ready {Done}/{Total} ({ObjectName}) ..."), Args);
			};
#endif // WITH_EDITOR
		// Reschedule everything to be executed at blocking priority, it bypasses the pause mechanism to ensure forward progress since we're waiting.
		// Important to keep using the asset compiling manager's thread pool to benefit from it's memory management and avoid OOM.
		FQueuedThreadPool* AssetThreadPool = FAssetCompilingManager::Get().GetThreadPool();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			Getter(Index).Reschedule(AssetThreadPool, EQueuedWorkPriority::Blocking);
		}

		const double MaxProcessingTimeSeconds = 0.016f;
		double StartTimeSeconds = FPlatformTime::Seconds();
		int32 NumDone = 0;
		TBitArray<> JobsToFinish(true, Num);
		TBitArray<> LoggedSlowTask(false, Num);
		for(;;)
		{
			for (TBitArray<>::FWordIterator It(JobsToFinish); It; ++It)
			{
				const uint32_t BaseIndex = It.GetIndex();
				uint32_t Word = It.GetWord();
				uint32_t WordJobState = Word;
				while (Word)
				{
					const uint32_t BitIndex = FBitSet::GetAndClearNextBit(Word);
					const uint32_t JobIndex = (BaseIndex * FBitSet::BitsPerWord) + BitIndex;

					ICompilable& Job = Getter(JobIndex);

					// If the job isn't complete, poll for completion
					if (Job.WaitCompletionWithTimeout(0.0f))
					{
						// Job is complete, mark the bit as done
						// and call our completion callback
						WordJobState &= ~(1u << BitIndex);
						PostCompileSingle(&Job);
						NumDone++;

#if WITH_EDITOR
						if (SlowTask.IsSet())
						{
							FText Progress = FormatProgress(NumDone, Num, Job.GetName());
							SlowTask->EnterProgressFrame(1.0f, Progress);
						}
#endif // WITH_EDITOR
						continue;
					}
				}
				It.SetWord(WordJobState);
			}

			if (NumDone >= Num)
			{
				break;
			}

			// We still have jobs inflight, so find an incomplete job and wait on it so we can be signalled if it ends before our timeout. 
			int IncompleteJobIndex = JobsToFinish.Find(true);
			check(IncompleteJobIndex != INDEX_NONE);
			ICompilable& IncompleteJob = Getter(IncompleteJobIndex);

			// If this job completes, we will clean it up on the next loop iteration
			float ElapsedTimeSeconds = float(FPlatformTime::Seconds() - StartTimeSeconds);
			if (!IncompleteJob.WaitCompletionWithTimeout(MaxProcessingTimeSeconds - ElapsedTimeSeconds))
			{
#if WITH_EDITOR
				// Progress the slow task so the editor may remain responsive
				if (SlowTask.IsSet())
				{
					FText Progress = FormatProgress(NumDone, Num, IncompleteJob.GetName());

					// Avoid spamming task progress while waiting
					if (!LoggedSlowTask[IncompleteJobIndex])
					{
						UE_LOG_REF(LogCategory, Display, TEXT("%s"), *Progress.ToString());
						LoggedSlowTask[IncompleteJobIndex] = true;
					}

					SlowTask->EnterProgressFrame(0.0f, Progress);
				}

				// SN-DBS jobs (which make use of the http service) needs to be ticked
				// from the game-thread to avoid starvation while we wait for other
				// async tasks to finish.
				if (GShaderCompilingManager && GShaderCompilingManager->IsCompiling())
				{
					const bool bLimitExecutionTime = true;
					const bool bBlockOnGlobalShaderCompletion = false;
					GShaderCompilingManager->ProcessAsyncResults(bLimitExecutionTime, bBlockOnGlobalShaderCompletion);
				}
#endif // WITH_EDITOR
				StartTimeSeconds = FPlatformTime::Seconds();
			}
		}

#if WITH_EDITOR
		SaveStallStack(FPlatformTime::Cycles64() - StartTime);
#endif
	}

#if WITH_EDITOR
	void EnsureInitializedCVars(
		const TCHAR* InName,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilation,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilationMaxConcurrency,
		FName InExperimentalSettingsName)
	{
		if (InExperimentalSettingsName != NAME_None)
		{
			// Early at boot, UEditorExperimentalSettings is unavailable, go straight to the ini file instead.
			bool bIsEnabled = false;
			if (GConfig->GetBool(TEXT("/Script/UnrealEd.EditorExperimentalSettings"), *InExperimentalSettingsName.ToString(), bIsEnabled, GEditorPerProjectIni))
			{
				InCVarAsyncCompilation->Set(bIsEnabled ? 1 : 0, ECVF_SetByProjectSetting);
			}
			
			// We cannot try to get the UEditorExperimentalSettings until all the class properties
			// have been linked otherwise the CDO wouldn't be able to initialize properties
			// from the ini files which would most likely end up with the default value of
			// async compilation being disabled. This obviously has the effect of async compilation
			// being disabled during boot until the object system is ready and settings can be read.
			FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase::ObjectSystemReady,
				[&InCVarAsyncCompilation, InExperimentalSettingsName]()
				{
					auto UpdateCVarFromSettings =
						[&InCVarAsyncCompilation, InExperimentalSettingsName]
						{
							const UEditorExperimentalSettings* Settings = GetDefault<UEditorExperimentalSettings>();
							FProperty* Property = Settings->GetClass()->FindPropertyByName(InExperimentalSettingsName);
							if (Property)
							{
								if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
								{
									bool bIsEnabled = BoolProperty->GetPropertyValue(BoolProperty->ContainerPtrToValuePtr<void>(Settings));
									InCVarAsyncCompilation->Set(bIsEnabled ? 1 : 0, ECVF_SetByProjectSetting);
								}
							}
						};

					GetMutableDefault<UEditorExperimentalSettings>()->OnSettingChanged().AddLambda(
						[UpdateCVarFromSettings, InExperimentalSettingsName](FName Name)
						{
							if (Name == InExperimentalSettingsName)
							{
								UpdateCVarFromSettings();
							}
						}
					);

					UpdateCVarFromSettings();
				}
			);
		}

		FString Value;
		if (FParse::Value(FCommandLine::Get(), *FString::Printf(TEXT("-async%scompilation="), InName), Value))
		{
			int32 AsyncCompilationValue = 0;
			if (Value == TEXT("1") || Value == TEXT("on"))
			{
				AsyncCompilationValue = 1;
			}

			if (Value == TEXT("2") || Value == TEXT("paused"))
			{
				AsyncCompilationValue = 2;
			}

			InCVarAsyncCompilation->Set(AsyncCompilationValue, ECVF_SetByCommandline);
		}

		int32 MaxConcurrency = -1;
		if (FParse::Value(FCommandLine::Get(), *FString::Printf(TEXT("-async%scompilationmaxconcurrency="), InName), MaxConcurrency))
		{
			InCVarAsyncCompilationMaxConcurrency->Set(MaxConcurrency, ECVF_SetByCommandline);
		}
	}

	void BindThreadPoolToCVar(
		FQueuedThreadPoolWrapper* InThreadPoolWrapper,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilation,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilationResume,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilationMaxConcurrency
		)
	{
		InCVarAsyncCompilation->SetOnChangedCallback(
			FConsoleVariableDelegate::CreateLambda(
				[InThreadPoolWrapper](IConsoleVariable* Variable)
				{
					if (Variable->GetInt() == 2)
					{
						InThreadPoolWrapper->Pause();
					}
					else
					{
						InThreadPoolWrapper->Resume();
					}
				}
			)
		);

		InCVarAsyncCompilationResume->SetOnChangedCallback(
			FConsoleVariableDelegate::CreateLambda(
				[InThreadPoolWrapper](IConsoleVariable* Variable)
				{
					if (Variable->GetInt() > 0)
					{
						InThreadPoolWrapper->Resume(Variable->GetInt());
					}
				}
			)
		);

		InCVarAsyncCompilationMaxConcurrency->SetOnChangedCallback(
			FConsoleVariableDelegate::CreateLambda(
				[InThreadPoolWrapper](IConsoleVariable* Variable)
				{
					InThreadPoolWrapper->SetMaxConcurrency(Variable->GetInt());
				}
			)
		);

		if (InCVarAsyncCompilation->GetInt() == 2)
		{
			InThreadPoolWrapper->Pause();
		}

		InThreadPoolWrapper->SetMaxConcurrency(InCVarAsyncCompilationMaxConcurrency->GetInt());
	}

	FAsyncCompilationStandardCVars::FAsyncCompilationStandardCVars(const TCHAR* AssetType, const TCHAR* AssetTypePluralLowerCase, const FConsoleCommandDelegate& FinishAllCommand)
		: AsyncCompilation(
			*FString::Printf(TEXT("Editor.Async%sCompilation"), AssetType),
			0,	// Constructor default is disabled, need to be activated by one of the activation method
			*FString::Printf(
				TEXT("1 - Async %s compilation is enabled.\n")
				TEXT("2 - Async %s compilation is enabled but on pause (for debugging).\n")
				TEXT("When enabled, %s will be replaced by placeholders until they are ready\n")
				TEXT("to reduce stalls on the game thread and improve overall editor performance."),
				AssetTypePluralLowerCase,
				AssetTypePluralLowerCase,
				AssetTypePluralLowerCase
			),
			ECVF_Default)
		, AsyncCompilationMaxConcurrency(
			*FString::Printf(TEXT("Editor.Async%sCompilationMaxConcurrency"), AssetType),
			-1,
			*FString::Printf(
				TEXT("Set the maximum number of concurrent %s compilation, -1 for unlimited."),
				AssetTypePluralLowerCase
			),
			ECVF_Default)
		, AsyncCompilationFinishAll(
			*FString::Printf(TEXT("Editor.Async%sCompilationFinishAll"), AssetType),
			*FString::Printf(TEXT("Finish all %s compilations"), AssetTypePluralLowerCase),
			FinishAllCommand)
		, AsyncCompilationResume(
			*FString::Printf(TEXT("Editor.Async%sCompilationResume"), AssetType),
			0,
			TEXT("Number of queued work to resume while paused."),
			ECVF_Default)
	{
	}

	struct FStackData
	{
		static const uint32 MaxDepth = 24;
		uint64 Backtrace[MaxDepth]{ 0 };
		uint64 Cycles = 0;
		int64  Count = 0;
	};

	static FRWLock                  BackTracesLock;
	static TMap<uint64, FStackData> BackTraces;

	void SaveStallStack(uint64 Cycles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AsyncCompilationHelpers::SaveStallStack);

		uint64 Backtrace[FStackData::MaxDepth];
		const uint32 Depth = FPlatformStackWalk::CaptureStackBackTrace(Backtrace, FStackData::MaxDepth);
		const uint64 StackHash = CityHash64(reinterpret_cast<const char*>(Backtrace), Depth * sizeof(Backtrace[0]));

		// We're already in an editor-only slow path, we don't mind the locking performance.
		FRWScopeLock Scope(BackTracesLock, SLT_Write);
		FStackData& StackData = BackTraces.FindOrAdd(StackHash);
		if (StackData.Count++ == 0)
		{
			FPlatformMemory::Memcpy(StackData.Backtrace, Backtrace, sizeof(StackData.Backtrace));
		}
		StackData.Cycles += Cycles;
	}

	void DumpStallStacks()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AsyncCompilationHelpers::DumpStallStacks);

		FRWScopeLock Scope(BackTracesLock, SLT_Write);
		TArray<FStackData*> Stacks;
		for (TPair<uint64, FStackData>& Pair : BackTraces)
		{
			Stacks.Add(&Pair.Value);
		}

		Algo::SortBy(Stacks, [](const FStackData* StackData) { return StackData->Cycles; });

		const int32 HumanReadableStringSize = 4096;
		ANSICHAR HumanReadableString[HumanReadableStringSize];
		int64 TotalCount = 0;

		uint64 TotalCycles = 0;
		for (FStackData* StackData : Stacks)
		{
			HumanReadableString[0] = '\0';

			// Start at index 2 to Skip both the CaptureBackTrace and this function
			for (int32 Index = 2; Index < FStackData::MaxDepth && StackData->Backtrace[Index]; ++Index)
			{
				FPlatformStackWalk::ProgramCounterToHumanReadableString(Index, StackData->Backtrace[Index], HumanReadableString, HumanReadableStringSize);
				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, LINE_TERMINATOR_ANSI);
			}

			TotalCount += StackData->Count;
			TotalCycles += StackData->Cycles;
			UE_LOGF(LogAsyncCompilation, Display, "Async Compilation Stall Stack: (Count: %llu, Time: %ls)\n%ls", StackData->Count, *FPlatformTime::PrettyTime(FPlatformTime::ToSeconds64(StackData->Cycles)), ANSI_TO_TCHAR(HumanReadableString));
		}

		if (TotalCount)
		{
			UE_LOGF(LogAsyncCompilation, Display, "Async Compilation Stall Stack: (Total Count: %llu, Total Time: %ls)", TotalCount, *FPlatformTime::PrettyTime(FPlatformTime::ToSeconds64(TotalCycles)));
		}
	}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
