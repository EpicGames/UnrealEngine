// Copyright Epic Games, Inc. All Rights Reserved.

#include "Atmosphere/AtmosphericFog.h"
#include "Async/TaskGraphInterfaces.h"
#include "Atmosphere/AtmosphericFogComponent.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/ArrowComponent.h"
#include "Engine/Texture2D.h"
#include "Atmosphere/AtmosphericFog.h"
#include "Components/BillboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AtmosphericFog)

#if WITH_EDITOR
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS

AAtmosphericFog::AAtmosphericFog(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AtmosphericFogComponent = CreateDefaultSubobject<UAtmosphericFogComponent>(TEXT("AtmosphericFogComponent0"));
	RootComponent = AtmosphericFogComponent;

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> FogTextureObject;
			FName ID_Fog;
			FText NAME_Fog;
			FConstructorStatics()
				: FogTextureObject(TEXT("/Engine/EditorResources/S_AtmosphericHeightFog"))
				, ID_Fog(TEXT("Fog"))
				, NAME_Fog(NSLOCTEXT("SpriteCategory", "Fog", "Fog"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (GetSpriteComponent())
		{
			GetSpriteComponent()->Sprite = ConstructorStatics.FogTextureObject.Get();
			GetSpriteComponent()->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			GetSpriteComponent()->SpriteInfo.Category = ConstructorStatics.ID_Fog;
			GetSpriteComponent()->SpriteInfo.DisplayName = ConstructorStatics.NAME_Fog;
			GetSpriteComponent()->SetupAttachment(AtmosphericFogComponent);
		}

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(150, 200, 255);

			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Fog;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Fog;
			ArrowComponent->SetupAttachment(AtmosphericFogComponent);
			ArrowComponent->bLightAttachment = true;
			ArrowComponent->bIsScreenSizeScaled = true;
		}
	}
#endif // WITH_EDITORONLY_DATA


	PrimaryActorTick.bCanEverTick = true;
	SetHidden(false);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

