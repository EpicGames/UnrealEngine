// Copyright Epic Games, Inc. All Rights Reserved.

#include "Atmosphere/AtmosphericFogComponent.h"
#include "Serialization/BulkData.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AtmosphericFogComponent)

#if WITH_EDITOR
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS 

//////////////////////////////////////////////////////////////////////////
// UAtmosphericFogComponent


UAtmosphericFogComponent::UAtmosphericFogComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsAtmosphericFog = true;
	SetPositionToMatchDeprecatedAtmosphericFog();
}

bool UAtmosphericFogComponent::IsPostLoadThreadSafe() const
{
	return true;
}

UAtmosphericFogComponent::~UAtmosphericFogComponent()
{
}

void UAtmosphericFogComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RemovedAtmosphericFog && Ar.IsLoading())
	{
		// Data use to serialize data in and drop them out.
		FByteBulkData TempTransmittanceData;
		FByteBulkData TempIrradianceData;
		FByteBulkData TempInscatterData;

		if (Ar.UEVer() >= VER_UE4_ATMOSPHERIC_FOG_CACHE_DATA)
		{
			TempTransmittanceData.Serialize(Ar, this, INDEX_NONE, false);
			TempIrradianceData.Serialize(Ar, this, INDEX_NONE, false);
		}

		TempInscatterData.Serialize(Ar, this, INDEX_NONE, false);

		int32 CounterVal = 0;
		Ar << CounterVal;

		SetPositionToMatchDeprecatedAtmosphericFog();

		// Old-style atmospheric fog didn't save a static lighting GUID
		// but also didn't interact with static baked lighting in the first place.
		// The USkyAtmosphericFogComponent ctor initializes its GUID to a random value,
		// which means we end up with cook non-determinims when that GUID doesn't get
		// replaced on load but does get saved in the exported data on cook. So
		// we initialize our static lighting GUID to a dummy GUID in this case.
		SetDummyStaticLightingGUIDs();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

