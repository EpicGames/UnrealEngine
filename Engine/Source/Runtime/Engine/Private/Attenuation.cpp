// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Attenuation.h"

#include "AudioDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Attenuation)


namespace
{
	static const float MinAttenuationValue   = 1.e-3f;
	static const float MinAttenuationValueDb = -60.0f;
} // namespace <>


FBaseAttenuationSettings::FBaseAttenuationSettings()
	: DistanceAlgorithm(EAttenuationDistanceModel::Linear)
	, AttenuationShape(EAttenuationShape::Sphere)
	, FalloffMode(ENaturalSoundFalloffMode::Continues)
	, dBAttenuationAtMax(MinAttenuationValueDb)
	, AttenuationShapeExtents(400.f, 0.f, 0.f)
	, ConeOffset(0.f)
	, FalloffDistance(3600.f)
	, ConeSphereRadius(0.f)
	, ConeSphereFalloffDistance(0.f)
{
}

FBaseAttenuationSettings::~FBaseAttenuationSettings() = default;

float FBaseAttenuationSettings::GetMaxDimension() const
{
	float MaxDimension = GetMaxFalloffDistance();

	switch (AttenuationShape)
	{
	case EAttenuationShape::Sphere:
	case EAttenuationShape::Cone:

		MaxDimension += AttenuationShapeExtents.X;
		break;

	case EAttenuationShape::Box:

		MaxDimension += FMath::Max3(AttenuationShapeExtents.X, AttenuationShapeExtents.Y, AttenuationShapeExtents.Z);
		break;

	case EAttenuationShape::Capsule:

		MaxDimension += FMath::Max(AttenuationShapeExtents.X, AttenuationShapeExtents.Y);
		break;

	default:
		check(false);
	}

	return FMath::Clamp(MaxDimension, 0.0f, static_cast<float>(FAudioDevice::GetMaxWorldDistance()));
}

float FBaseAttenuationSettings::GetMaxFalloffDistance() const
{
	static const float WorldMax = static_cast<float>(FAudioDevice::GetMaxWorldDistance());
	if (FalloffDistance > WorldMax)
	{
		return WorldMax;
	}

	float MaxFalloffDistance = FalloffDistance;
	switch (DistanceAlgorithm)
	{
		case EAttenuationDistanceModel::Custom:
		{
			const FRichCurve* Curve = CustomAttenuationCurve.GetRichCurveConst();
			check(Curve);

			float LastTime = 0.0f;
			const FRichCurveKey* LastKey = nullptr;
			for (const FRichCurveKey& Key : Curve->Keys)
			{
				if (Key.Time > LastTime)
				{
					LastTime = Key.Time;
					LastKey = &Key;
				}
			}

			const float MaxValue = LastKey ? FMath::Max(LastKey->Value, 0.0f) : 0.0f;

			// If last key's distance is near zero, scale the falloff distance accordingly
			if (FMath::IsNearlyZero(MaxValue, MinAttenuationValue))
			{
				MaxFalloffDistance *= LastTime;
			}
			// Otherwise, curve never terminates to non-zero value, so return WorldMax
			else
			{
				MaxFalloffDistance = WorldMax;
			}
		}
		break;

		case EAttenuationDistanceModel::NaturalSound:
		{
			switch (FalloffMode)
			{
				case ENaturalSoundFalloffMode::Hold:
				{
					MaxFalloffDistance = WorldMax;
				}
				break;

				case ENaturalSoundFalloffMode::Continues:
				{
					MaxFalloffDistance = FalloffDistance * MinAttenuationValueDb / FMath::Min(dBAttenuationAtMax, -UE_KINDA_SMALL_NUMBER);
				}
				break;

				case ENaturalSoundFalloffMode::Silent:
				default:
				break;
			}
		}
		break;

		// All of these cases scale over the provided FalloffDistance, so just return that as max
		case EAttenuationDistanceModel::Inverse:
		case EAttenuationDistanceModel::Linear:
		case EAttenuationDistanceModel::Logarithmic:
		case EAttenuationDistanceModel::LogReverse:
		default:
		break;
	}

	return MaxFalloffDistance;
}

float FBaseAttenuationSettings::Evaluate(const FTransform& Origin, const FVector Location, const float DistanceScale) const
{
	float AttenuationMultiplier = 1.f;

	switch (AttenuationShape)
	{
	case EAttenuationShape::Sphere:
	{
		const float Distance = FMath::Max<float>(FVector::Dist( Origin.GetTranslation(), Location ) - AttenuationShapeExtents.X, 0.f);
		AttenuationMultiplier = AttenuationEval(Distance, FalloffDistance, DistanceScale);
		break;
	}

	case EAttenuationShape::Box:
		AttenuationMultiplier = AttenuationEvalBox(Origin, Location, DistanceScale);
		break;

	case EAttenuationShape::Capsule:
		AttenuationMultiplier = AttenuationEvalCapsule(Origin, Location, DistanceScale);
		break;

	case EAttenuationShape::Cone:
		AttenuationMultiplier = AttenuationEvalCone(Origin, Location, DistanceScale);
		break;

	default:
		check(false);
	}

	return AttenuationMultiplier;
}

float FBaseAttenuationSettings::AttenuationEval(const float Distance, const float Falloff, const float DistanceScale) const
{
	// Clamp the input distance between 0.0f and Falloff. If the Distance
	// is actually less than the min value, it will use the min-value of the algorithm/curve
	// rather than assume it's 1.0 (i.e. it could be 0.0 for an inverse curve). Similarly, if the distance
	// is greater than the falloff value, it'll use the algorithm/curve value evaluated at Falloff distance,
	// which could be 1.0 (and not 0.0f).

	const float FalloffCopy = FMath::Max(Falloff, 1.0f);
	float DistanceCopy = Distance * DistanceScale;

	float Result = 0.0f;
	switch (DistanceAlgorithm)
	{
		case EAttenuationDistanceModel::Linear:

			Result = (1.0f - (DistanceCopy / FalloffCopy));
			break;

		case EAttenuationDistanceModel::Logarithmic:
			{
				DistanceCopy = FMath::Max(DistanceCopy, UE_KINDA_SMALL_NUMBER);
				Result = 0.5f * -FMath::Loge(DistanceCopy / FalloffCopy);
			}
			break;

		case EAttenuationDistanceModel::Inverse:
			{
				DistanceCopy = FMath::Max(DistanceCopy, UE_KINDA_SMALL_NUMBER);
				Result = 0.02f / (DistanceCopy / FalloffCopy);
			}
			break;

		case EAttenuationDistanceModel::LogReverse:
		{
			if (DistanceCopy > FalloffCopy)
			{
				Result = 0.0f;
			}
			else
			{
				const float Argument = FMath::Max(1.0f - (DistanceCopy / FalloffCopy), UE_KINDA_SMALL_NUMBER);
				Result = 1.0f + 0.5f * FMath::Loge(Argument);
			}
		}
		break;

		case EAttenuationDistanceModel::NaturalSound:
		{
			check(dBAttenuationAtMax <= 0.0f);
			float Alpha = DistanceCopy / FalloffCopy;
			if (FalloffMode == ENaturalSoundFalloffMode::Hold)
			{
				Alpha = FMath::Clamp(Alpha, 0.0f, 1.0f);
				Result = Audio::ConvertToLinear(Alpha * dBAttenuationAtMax);
			}
			else if (FalloffMode == ENaturalSoundFalloffMode::Silent)
			{
				Alpha = FMath::Clamp(Alpha, 0.0f, 1.0f);
				if (Alpha < 1.0f)
				{
					Result = Audio::ConvertToLinear(Alpha * dBAttenuationAtMax);
				}
				else
				{
					Result = 0.0f;
				}
			}
			else
			{
				Result = Audio::ConvertToLinear(Alpha * dBAttenuationAtMax);
			}
			break;
		}

		case EAttenuationDistanceModel::Custom:

			Result = CustomAttenuationCurve.GetRichCurveConst()->Eval(DistanceCopy / FalloffCopy);
			break;

		default:
			checkf(false, TEXT("Unknown attenuation distance algorithm!"))
			break;
	}

	// Make sure the output is clamped between 0.0 and 1.0f. Some of the algorithms above can
	// result in bad values at the edges.
	return FMath::Clamp(Result, 0.0f, 1.0f);
}

float FBaseAttenuationSettings::AttenuationEvalBox(const FTransform& Origin, const FVector Location, const float DistanceScale) const
{
	const float DistanceSq = ComputeSquaredDistanceFromBoxToPoint(-AttenuationShapeExtents, AttenuationShapeExtents, Origin.InverseTransformPositionNoScale(Location));
	if (FMath::IsNearlyZero(DistanceSq) || DistanceSq < FalloffDistance * FalloffDistance)
	{ 
		return AttenuationEval(FMath::Sqrt(DistanceSq), FalloffDistance, DistanceScale);
	}

	return 0.f;
}

float FBaseAttenuationSettings::AttenuationEvalCapsule(const FTransform& Origin, const FVector Location, const float DistanceScale) const
{
	float Distance = 0.f;
	const float CapsuleHalfHeight = AttenuationShapeExtents.X;
	const float CapsuleRadius = AttenuationShapeExtents.Y;

	// Capsule devolves to a sphere if HalfHeight <= Radius
	if (CapsuleHalfHeight <= CapsuleRadius )
	{
		Distance = FMath::Max<FVector::FReal>(FVector::Dist( Origin.GetTranslation(), Location ) - CapsuleRadius, 0.f);
	}
	else
	{
		const FVector PointOffset = (CapsuleHalfHeight - CapsuleRadius) * Origin.GetUnitAxis( EAxis::Z );
		const FVector StartPoint = Origin.GetTranslation() + PointOffset;
		const FVector EndPoint = Origin.GetTranslation() - PointOffset;

		Distance = FMath::PointDistToSegment(Location, StartPoint, EndPoint) - CapsuleRadius;
	}

	return AttenuationEval(Distance, FalloffDistance, DistanceScale);
}

float FBaseAttenuationSettings::AttenuationEvalCone(const FTransform& Origin, const FVector Location, const float DistanceScale) const
{
	const FVector Forward = Origin.GetUnitAxis( EAxis::X );

	float AttenuationMultiplier = 1.f;
	float SphereAttenuationMultiplier = 0.f;

	const FVector ConeOrigin = Origin.GetTranslation() - (Forward * ConeOffset);

	// Evaluate sphere attenuation If ConeSphereRadius is nonzero
	if (!FMath::IsNearlyZero(ConeSphereRadius))
	{
		const float SphereDistance = FMath::Max<float>(FVector::Dist(ConeOrigin, Location) - ConeSphereRadius, 0.f);
		SphereAttenuationMultiplier = AttenuationEval(SphereDistance, ConeSphereFalloffDistance, DistanceScale);
	}

	// Cone devolves into sphere check if ConeSphereRadius >= AttenuationShapeExtents.X
	if (ConeSphereRadius >= AttenuationShapeExtents.X)
	{
		AttenuationMultiplier = SphereAttenuationMultiplier;
	}
	else
	{
		const float Distance = FMath::Max(FVector::Dist(ConeOrigin, Location) - AttenuationShapeExtents.X, 0.f);
		AttenuationMultiplier *= AttenuationEval(Distance, FalloffDistance, DistanceScale);

		if (AttenuationMultiplier > 0.f)
		{
			const float theta = FMath::RadiansToDegrees(FMath::Abs(FMath::Acos(FVector::DotProduct(Forward, (Location - ConeOrigin).GetSafeNormal()))));
			AttenuationMultiplier *= AttenuationEval(theta - AttenuationShapeExtents.Y, AttenuationShapeExtents.Z, 1.0f);
		}

		AttenuationMultiplier = FMath::Max(AttenuationMultiplier, SphereAttenuationMultiplier);
	}

	return AttenuationMultiplier;
}

void FBaseAttenuationSettings::CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, AttenuationShapeDetails>& ShapeDetailsMap) const
{
	AttenuationShapeDetails ShapeDetails;
	ShapeDetails.Extents = AttenuationShapeExtents;
	ShapeDetails.Falloff = FalloffDistance;
	ShapeDetails.ConeOffset = ConeOffset;
	ShapeDetails.ConeSphereRadius = ConeSphereRadius;
	ShapeDetails.ConeSphereFalloff = ConeSphereFalloffDistance;

	ShapeDetailsMap.Add(AttenuationShape, MoveTemp(ShapeDetails));
}
