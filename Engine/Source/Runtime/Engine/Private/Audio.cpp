// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Audio.cpp: Unreal base audio.
=============================================================================*/

#include "Audio.h"
#include "Algo/Find.h"
#include "Audio/AudioDebug.h"
#include "AudioDevice.h"
#include "AudioPluginUtilities.h"
#include "Components/SynthComponent.h"
#include "Engine/Engine.h"
#include "EngineAnalytics.h"
#include "IAnalyticsProviderET.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Sound/AudioOutputTarget.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "UObject/UObjectIterator.h"
#include "XmlFile.h"
#include "XmlNode.h"
#include "Algo/ForEach.h"

#ifndef WITH_SNDFILE_IO
#define WITH_SNDFILE_IO (0)
#endif //WITH_SNDFILE_IO

DEFINE_LOG_CATEGORY(LogAudio);

DEFINE_LOG_CATEGORY(LogAudioDebug);

/** Audio stats */

DEFINE_STAT(STAT_AudioMemorySize);
DEFINE_STAT(STAT_ActiveSounds);
DEFINE_STAT(STAT_AudioSources);
DEFINE_STAT(STAT_AudioVirtualLoops);
DEFINE_STAT(STAT_WaveInstances);
DEFINE_STAT(STAT_WavesDroppedDueToPriority);
DEFINE_STAT(STAT_AudioMaxChannels);
DEFINE_STAT(STAT_AudioMaxStoppingSources);
DEFINE_STAT(STAT_AudibleWavesDroppedDueToPriority);
DEFINE_STAT(STAT_AudioFinishedDelegatesCalled);
DEFINE_STAT(STAT_AudioFinishedDelegates);
DEFINE_STAT(STAT_AudioBufferTime);
DEFINE_STAT(STAT_AudioBufferTimeChannels);

DEFINE_STAT(STAT_AudioDecompressTime);
DEFINE_STAT(STAT_AudioPrepareDecompressionTime);
DEFINE_STAT(STAT_AudioStreamedDecompressTime);

DEFINE_STAT(STAT_AudioUpdateEffects);
DEFINE_STAT(STAT_AudioEvaluateConcurrency);
DEFINE_STAT(STAT_AudioUpdateSources);
DEFINE_STAT(STAT_AudioResourceCreationTime);
DEFINE_STAT(STAT_AudioSourceInitTime);
DEFINE_STAT(STAT_AudioSourceCreateTime);
DEFINE_STAT(STAT_AudioSubmitBuffersTime);
DEFINE_STAT(STAT_AudioStartSources);
DEFINE_STAT(STAT_AudioGatherWaveInstances);
DEFINE_STAT(STAT_AudioFindNearestLocation);

/** CVars */
static int32 DisableStereoSpreadCvar = 0;
FAutoConsoleVariableRef CVarDisableStereoSpread(
	TEXT("au.DisableStereoSpread"),
	DisableStereoSpreadCvar,
	TEXT("When set to 1, ignores the 3D Stereo Spread property in attenuation settings and instead renders audio from a singular point.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 AllowAudioSpatializationCVar = 1;
FAutoConsoleVariableRef CVarAllowAudioSpatializationCVar(
	TEXT("au.AllowAudioSpatialization"),
	AllowAudioSpatializationCVar,
	TEXT("Controls if we allow spatialization of audio, normally this is enabled.  If disabled all audio won't be spatialized, but will have attenuation.\n")
	TEXT("0: Disable, >0: Enable"),
	ECVF_Default);

static int32 OcclusionFilterScaleEnabledCVar = 0;
FAutoConsoleVariableRef CVarOcclusionFilterScaleEnabled(
	TEXT("au.EnableOcclusionFilterScale"),
	OcclusionFilterScaleEnabledCVar,
	TEXT("Whether or not we scale occlusion by 0.25f to compensate for change in filter cutoff frequencies in audio mixer. \n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 BypassPlayWhenSilentCVar = 0;
FAutoConsoleVariableRef CVarBypassPlayWhenSilent(
	TEXT("au.BypassPlayWhenSilent"),
	BypassPlayWhenSilentCVar,
	TEXT("When set to 1, ignores the Play When Silent flag for non-procedural sources.\n")
	TEXT("0: Honor the Play When Silent flag, 1: stop all silent non-procedural sources."),
	ECVF_Default);

static float WaveInstanceMinVolumeThresholdCVar = UE_KINDA_SMALL_NUMBER;
FAutoConsoleVariableRef CVarMinVolumeThreshold(
	TEXT("au.WaveInstanceMinVolume"),
	WaveInstanceMinVolumeThresholdCVar,
	TEXT("Sets the minimum volume for a wave instance to be considered active\n")
	TEXT("Default is 0.0001 (-80 dB)"),
	ECVF_Default);

namespace AudioModulationPrivate
{
	static const FLazyName VolumeParameterName("Volume");
	static const FLazyName PitchParameterName("Pitch");
	static const FLazyName HighpassParameterName("HPFCutoffFrequency");
	static const FLazyName LowpassParameterName("LPFCutoffFrequency");
}

bool IsAudioPluginEnabled(EAudioPlugin PluginType)
{
	switch (PluginType)
	{
	case EAudioPlugin::SPATIALIZATION:
		return AudioPluginUtilities::GetDesiredSpatializationPlugin() != nullptr;
	case EAudioPlugin::REVERB:
		return AudioPluginUtilities::GetDesiredReverbPlugin() != nullptr;
	case EAudioPlugin::OCCLUSION:
		return AudioPluginUtilities::GetDesiredOcclusionPlugin() != nullptr;
	case EAudioPlugin::MODULATION:
		return AudioPluginUtilities::GetDesiredModulationPlugin() != nullptr;
	default:
		return false;
	}
}

UClass* GetAudioPluginCustomSettingsClass(EAudioPlugin PluginType)
{
	switch (PluginType)
	{
		case EAudioPlugin::SPATIALIZATION:
		{
			if (IAudioSpatializationFactory* Factory = AudioPluginUtilities::GetDesiredSpatializationPlugin())
			{
				return Factory->GetCustomSpatializationSettingsClass();
			}
		}
		break;

		case EAudioPlugin::REVERB:
		{
			if (IAudioReverbFactory* Factory = AudioPluginUtilities::GetDesiredReverbPlugin())
			{
				return Factory->GetCustomReverbSettingsClass();
			}
		}
		break;

		case EAudioPlugin::OCCLUSION:
		{
			if (IAudioOcclusionFactory* Factory = AudioPluginUtilities::GetDesiredOcclusionPlugin())
			{
				return Factory->GetCustomOcclusionSettingsClass();
			}
		}
		break;

		case EAudioPlugin::MODULATION:
		{
			return nullptr;
		}

		case EAudioPlugin::SOURCEDATAOVERRIDE:
		{
			if (IAudioSourceDataOverrideFactory* Factory = AudioPluginUtilities::GetDesiredSourceDataOverridePlugin())
			{
				return Factory->GetCustomSourceDataOverrideSettingsClass();
			}		
		}
		break;

		default:
			static_assert(static_cast<uint32>(EAudioPlugin::COUNT) == 5, "Possible missing audio plugin type case coverage");
		break;
	}

	return nullptr;
}

bool IsSpatializationCVarEnabled()
{
	return AllowAudioSpatializationCVar != 0;
}

float FWaveInstance::GetEffectiveModulationValue(const EModulationDestination InDestination) const
{
	FSoundModulationDestinationSettings Settings;
	float OutValue;
	CalculateInheritedModulationSettingsAndValue(InDestination, Settings, OutValue);

	return OutValue;
}

FSoundModulationDestinationSettings FWaveInstance::GetEffectiveModulationSettings(const EModulationDestination InDestination) const
{
	FSoundModulationDestinationSettings Settings;
	float OutValue;
	CalculateInheritedModulationSettingsAndValue(InDestination, Settings, OutValue);

	return Settings;
}

Audio::FModulationParameter FWaveInstance::GetParameterBasedOnDestination(const EModulationDestination InDestination)
{
	switch (InDestination)
	{
	case EModulationDestination::Volume:
		if (const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(AudioModulationPrivate::VolumeParameterName))
		{
			return *Parameter;
		}
		break;
	case EModulationDestination::Pitch:
		if (const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(AudioModulationPrivate::PitchParameterName))
		{
			return *Parameter;
		}
		break;
	case EModulationDestination::Highpass:
		if (const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(AudioModulationPrivate::HighpassParameterName))
		{
			return *Parameter;
		}
		break;
	case EModulationDestination::Lowpass:
		if (const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(AudioModulationPrivate::LowpassParameterName))
		{
			return *Parameter;
		}
		break;
	default:
		break;
	}
	return Audio::GetDefaultModulationParameter();
}

void FWaveInstance::CalculateInheritedModulationSettingsAndValue(const EModulationDestination InDestination, FSoundModulationDestinationSettings& InOutSettings, float& OutValue) const
{
	const Audio::FModulationParameter& InParam = GetParameterBasedOnDestination(InDestination);
	
	OutValue = InParam.DefaultValue;

	const FActiveSound& InActiveSound = *ActiveSound;
	const FSoundModulationDestinationSettings& InWaveSettings = WaveData->ModulationSettings.GetSettingsForDestination(InDestination);
	EModulationRouting InWaveRouting = WaveData->ModulationSettings.GetRoutingForDestination(InDestination);

	const USoundClass* EffectiveSoundClass = InActiveSound.GetSoundClass();
	if (SoundClass)
	{
		EffectiveSoundClass = SoundClass;
	}
	
	auto MixInRoutedValue = [&InParam](float& InOutValueA, float InValueB)
	{
		if (InParam.bRequiresConversion)
		{
			InParam.NormalizedFunction(InOutValueA);
			InParam.NormalizedFunction(InValueB);
		}
		InParam.MixFunction(InOutValueA, InValueB);
		if (InParam.bRequiresConversion)
		{
			InParam.UnitFunction(InOutValueA);
		}
	};
	
	auto UnionSoundClassSettings = [&](FSoundModulationDestinationSettings& Settings)
		{
			if (EffectiveSoundClass)
			{
				const FSoundModulationDestinationSettings& ClassSettings = EffectiveSoundClass->Properties.ModulationSettings.GetSettingsForDestination(InDestination);
				OutValue = Settings.Value;
				MixInRoutedValue(OutValue, ClassSettings.Value);
				Settings.Modulators = Settings.Modulators.Union(ClassSettings.Modulators);
			}
		};

	switch (InActiveSound.ModulationRouting.GetRoutingForDestination(InDestination))
	{
	case EModulationRouting::Union:
	{
		FSoundModulationDestinationSettings UnionSettings = InActiveSound.ModulationRouting.GetSettingsForDestination(InDestination);
		switch (InWaveRouting)
		{
		case EModulationRouting::Union:
		{
			OutValue = UnionSettings.Value;
			MixInRoutedValue(OutValue, InWaveSettings.Value);
			UnionSettings.Modulators = UnionSettings.Modulators.Union(InWaveSettings.Modulators);
			UnionSoundClassSettings(UnionSettings);
		}
		break;

		case EModulationRouting::Inherit:
		{
			UnionSoundClassSettings(UnionSettings);
		}
		break;

		case EModulationRouting::Override:
		{
			OutValue = UnionSettings.Value;
			MixInRoutedValue(OutValue, InWaveSettings.Value);
			UnionSettings.Modulators = UnionSettings.Modulators.Union(InWaveSettings.Modulators);
		}
		break;

		case EModulationRouting::Disable:
		default:
			break;
		}

		InOutSettings = UnionSettings;
		OutValue = UnionSettings.Value;
		return;
	}

	case EModulationRouting::Inherit:
	{
		switch (InWaveRouting)
		{
		case EModulationRouting::Union:
		{
			FSoundModulationDestinationSettings UnionSettings = InWaveSettings;
			UnionSoundClassSettings(UnionSettings);
			InOutSettings = UnionSettings;
			OutValue = UnionSettings.Value;
		}
		break;

		case EModulationRouting::Inherit:
		{
			if (EffectiveSoundClass)
			{
				FSoundModulationDestinationSettings InheritSettings = EffectiveSoundClass->Properties.ModulationSettings.GetSettingsForDestination(InDestination);
				InOutSettings = InheritSettings;
				OutValue = InheritSettings.Value;
			}
		}
		break;

		case EModulationRouting::Override:
		{
			InOutSettings = InWaveSettings;
			OutValue = InWaveSettings.Value;
		}

		case EModulationRouting::Disable:
		default:
			break;
		}
	}
	break;

	case EModulationRouting::Override:
	{
		FSoundModulationDestinationSettings OverrideSettings = InActiveSound.ModulationRouting.GetSettingsForDestination(InDestination);
		InOutSettings = OverrideSettings;
		OutValue = OverrideSettings.Value;	
	}
	break;

	case EModulationRouting::Disable:
	default:
		break;
	}
}

/*-----------------------------------------------------------------------------
	FSoundBuffer implementation.
-----------------------------------------------------------------------------*/

FSoundBuffer::~FSoundBuffer()
{
}

/**
 * This will return the name of the SoundClass of the Sound that this buffer(SoundWave) belongs to.
 * NOTE: This will find the first cue in the ObjectIterator list.  So if we are using SoundWaves in multiple
 * places we will pick up the first one only.
 **/
FName FSoundBuffer::GetSoundClassName()
{
	// need to look in all cues
	for (TObjectIterator<USoundBase> It; It; ++It)
	{
		USoundCue* Cue = Cast<USoundCue>(*It);
		if (Cue)
		{
			// get all the waves this cue uses
			TArray<USoundNodeWavePlayer*> WavePlayers;
			Cue->RecursiveFindNode<USoundNodeWavePlayer>(Cue->FirstNode, WavePlayers);

			// look through them to see if this cue uses a wave this buffer is bound to, via ResourceID
			for (int32 WaveIndex = 0; WaveIndex < WavePlayers.Num(); ++WaveIndex)
			{
				USoundWave* WaveNode = WavePlayers[WaveIndex]->GetSoundWave();
				if (WaveNode != NULL)
				{
					if (WaveNode->ResourceID == ResourceID)
					{
						if (Cue->GetSoundClass())
						{
							return Cue->GetSoundClass()->GetFName();
						}
						else
						{
							return NAME_None;
						}
					}
				}
			}
		}
		else
		{
			USoundWave* Wave = Cast<USoundWave>(*It);
			if (Wave && Wave->ResourceID == ResourceID)
			{
				if (Wave->GetSoundClass())
				{
					return Wave->GetSoundClass()->GetFName();
				}
				else
				{
					return NAME_None;
				}
			}
		}
	}

	return NAME_None;
}

FString FSoundBuffer::GetChannelsDesc()
{
	switch (NumChannels)
	{
		case 1:
			return FString("Mono");
		case 2:
			return FString("Stereo");
		case 6:
			return FString("5.1");
		case 8:
			return FString("7.1");
		default:
			return FString::Printf(TEXT("%d Channels"), NumChannels);
	}
}

FString FSoundBuffer::Describe(bool bUseLongName)
{
	// format info string
	const FName SoundClassName = GetSoundClassName();
	FString AllocationString = bAllocationInPermanentPool ? TEXT("Permanent, ") : TEXT("");
	FString ChannelsDesc = GetChannelsDesc();
	FString SoundName = bUseLongName ? ResourceName : FPaths::GetExtension(ResourceName);

	return FString::Printf(TEXT("%8.2fkb, %s%s, '%s', Class: %s"), GetSize() / 1024.0f, *AllocationString, *ChannelsDesc, *ResourceName, *SoundClassName.ToString());
}

/*-----------------------------------------------------------------------------
	FSoundSource implementation.
-----------------------------------------------------------------------------*/

FSoundSource::~FSoundSource() = default;

FString FSoundSource::Describe(bool bUseLongName)
{
	return FString::Printf(TEXT("Wave: %s, Volume: %6.2f, Owner: %s"),
		bUseLongName ? *WaveInstance->WaveData->GetPathName() : *WaveInstance->WaveData->GetName(),
		WaveInstance->GetVolume(),
		WaveInstance->ActiveSound ? *WaveInstance->ActiveSound->GetOwnerName() : TEXT("None"));
}

void FSoundSource::Stop()
{
	if (WaveInstance)
	{
		// The sound is stopping, so set the envelope value to 0.0f
		WaveInstance->SetEnvelopeValue(0.0f);
		NotifyPlaybackData();

		check(AudioDevice);
		AudioDevice->WaveInstanceSourceMap.Remove(WaveInstance);
		WaveInstance->NotifyFinished(true);
		WaveInstance = nullptr;
	}

	// Remove this source from free list regardless of if this had a wave instance created
	AudioDevice->FreeSources.AddUnique(this);
}

void FSoundSource::SetPauseByGame(bool bInIsPauseByGame)
{
	bIsPausedByGame = bInIsPauseByGame;
	UpdatePause();
}

void FSoundSource::SetPauseManually(bool bInIsPauseManually)
{
	bIsManuallyPaused = bInIsPauseManually;
	UpdatePause();
}

void FSoundSource::UpdatePause()
{
	if (IsPaused() && !bIsPausedByGame && !bIsManuallyPaused)
	{
		Play();
	}
	else if (!IsPaused() && (bIsManuallyPaused || bIsPausedByGame))
	{
		Pause();
	}
}

bool FSoundSource::IsGameOnly() const
{
	return (WaveInstance && !WaveInstance->bIsUISound);
}

bool FSoundSource::SetReverbApplied(bool bHardwareAvailable)
{
	// TODO: REMOVE THIS WHEN LEGACY BACKENDS ARE DELETED
	
	// Do not apply reverb if it is explicitly disallowed
	bReverbApplied = WaveInstance->bReverb && bHardwareAvailable;

	// Do not apply reverb to music
	if (WaveInstance->bIsMusic)
	{
		bReverbApplied = false;
	}

	return(bReverbApplied);
}

float FSoundSource::SetLFEBleed()
{
	LFEBleed = WaveInstance->LFEBleed;

	if (AudioDevice->GetMixDebugState() == DEBUGSTATE_TestLFEBleed)
	{
		LFEBleed = 10.0f;
	}

	return LFEBleed;
}

void FSoundSource::SetFilterFrequency()
{
	// HPF is only available with audio mixer enabled
	switch (AudioDevice->GetMixDebugState())
	{
		case DEBUGSTATE_TestLPF:
		{
			LPFFrequency = MIN_FILTER_FREQUENCY;
		}
		break;

		case DEBUGSTATE_DisableLPF:
		{
			LPFFrequency = MAX_FILTER_FREQUENCY;
		}
		break;

		default:
		{
			// compensate for filter coefficient calculation error for occlusion
			float OcclusionFilterScale = 1.0f;
			if (OcclusionFilterScaleEnabledCVar == 1 && !FMath::IsNearlyEqual(WaveInstance->OcclusionFilterFrequency, MAX_FILTER_FREQUENCY))
			{
				OcclusionFilterScale = 0.25f;
			}

			// Set the LPFFrequency to lowest provided value
			LPFFrequency = WaveInstance->OcclusionFilterFrequency * OcclusionFilterScale;

			if (WaveInstance->bEnableLowPassFilter)
			{
				LPFFrequency = FMath::Min(LPFFrequency, WaveInstance->LowPassFilterFrequency);
			}

			LPFFrequency = FMath::Min(LPFFrequency, WaveInstance->AmbientZoneFilterFrequency);
			LPFFrequency = FMath::Min(LPFFrequency, WaveInstance->AttenuationLowpassFilterFrequency);
			LPFFrequency = FMath::Min(LPFFrequency, WaveInstance->SoundClassFilterFrequency);
		}
		break;
	}

	// HPF is only available with audio mixer enabled
	switch (AudioDevice->GetMixDebugState())
	{
		case DEBUGSTATE_TestHPF:
		{
			HPFFrequency = MAX_FILTER_FREQUENCY;
		}
		break;

		case DEBUGSTATE_DisableHPF:
		{
			HPFFrequency = MIN_FILTER_FREQUENCY;
		}
		break;

		default:
		{
			// Set the HPFFrequency to highest provided value
			HPFFrequency = WaveInstance->AttenuationHighpassFilterFrequency;

			if (WaveInstance->bEnableHighPassFilter)
			{
				HPFFrequency = FMath::Max(HPFFrequency, WaveInstance->HighPassFilterFrequency);
			}
		}
		break;
	}
}

void FSoundSource::UpdateStereoEmitterPositions()
{
	// Only call this function if we're told to use spatialization
	check(WaveInstance->GetUseSpatialization());
	check(GetNumChannels() == 2);

	if (!DisableStereoSpreadCvar && WaveInstance->StereoSpread > 0.0f)
	{
		// We need to compute the stereo left/right channel positions using the audio component position and the spread
		FVector ListenerPosition;

		const bool bAllowAttenuationOverride = false;
		const int32 ListenerIndex = WaveInstance->ActiveSound ? WaveInstance->ActiveSound->GetClosestListenerIndex() : 0;
		AudioDevice->GetListenerPosition(ListenerIndex, ListenerPosition, bAllowAttenuationOverride);
		FVector ListenerToSourceDir = (WaveInstance->Location - ListenerPosition).GetSafeNormal();

		float HalfSpread = 0.5f * WaveInstance->StereoSpread;

		// Get direction of left emitter from true emitter position (left hand rule)
		FVector LeftEmitterDir = FVector::CrossProduct(ListenerToSourceDir, FVector::UpVector);
		FVector LeftEmitterOffset = LeftEmitterDir * HalfSpread;

		// Get position vector of left emitter by adding to true emitter the dir scaled by half the spread
		LeftChannelSourceLocation = WaveInstance->Location + LeftEmitterOffset;

		// Right emitter position is same as right but opposite direction
		RightChannelSourceLocation = WaveInstance->Location - LeftEmitterOffset;
	}
	else
	{
		LeftChannelSourceLocation = WaveInstance->Location;
		RightChannelSourceLocation = WaveInstance->Location;
	}
}

float FSoundSource::GetDebugVolume(const float InVolume)
{
	float OutVolume = InVolume;

#if ENABLE_AUDIO_DEBUG

	// Bail if we don't have a device manager.
	if (!GEngine || !GEngine->GetAudioDeviceManager() || !WaveInstance || !DebugInfo.IsValid() )
	{
		return OutVolume;
	}

	// Solos/Mutes (dev only).
	Audio::FAudioDebugger& Debugger = GEngine->GetAudioDeviceManager()->GetDebugger();	
	FDebugInfo Info;
				
	// SoundWave Solo/Mutes.
	if (OutVolume != 0.0f)
	{
		Debugger.QuerySoloMuteSoundWave(WaveInstance->GetName(), Info.bIsSoloed, Info.bIsMuted, Info.MuteSoloReason);
		if (Info.bIsMuted)
		{
			OutVolume = 0.0f;
		}
	}

	// SoundCues mutes/solos (not strictly just cues but any SoundBase)
	if (OutVolume != 0.0f && WaveInstance->ActiveSound)
	{						
		if (USoundBase* ActiveSound= WaveInstance->ActiveSound->GetSound())
		{
			Debugger.QuerySoloMuteSoundCue(ActiveSound->GetName(), Info.bIsSoloed, Info.bIsMuted, Info.MuteSoloReason);
			if (Info.bIsMuted)
			{
				OutVolume = 0.0f;
			}
		}
	}

	// SoundClass mutes/solos.
	if (OutVolume != 0.0f && WaveInstance->SoundClass)
	{
		FString SoundClassName;
		WaveInstance->SoundClass->GetName(SoundClassName);
		Debugger.QuerySoloMuteSoundClass(SoundClassName, Info.bIsSoloed, Info.bIsMuted, Info.MuteSoloReason);
		if (Info.bIsMuted)
		{
			OutVolume = 0.0f;
		}
	}

	// Update State. 
	FScopeLock Lock(&DebugInfo->CS);
	{
		DebugInfo->bIsMuted = Info.bIsMuted;
		DebugInfo->bIsSoloed = Info.bIsSoloed;
		DebugInfo->MuteSoloReason = MoveTemp(Info.MuteSoloReason);
	}

#endif //ENABLE_AUDIO_DEBUG

	return OutVolume;
}

FSpatializationParams FSoundSource::GetSpatializationParams()
{
	FSpatializationParams Params;

	// Put the audio time stamp on the spatialization params
	Params.AudioClock = AudioDevice->GetAudioTime();

	if (WaveInstance->GetUseSpatialization())
	{
		FVector EmitterPosition = AudioDevice->GetListenerTransformedDirection(WaveInstance->Location, &Params.Distance);

		// Independently retrieve the attenuation used for distance in case it was overridden
		Params.AttenuationDistance = AudioDevice->GetDistanceToNearestListener(WaveInstance->Location);
		
		// If we are using the Non-spatialized radius feature
		if (WaveInstance->NonSpatializedRadiusStart > 0.0f)
		{
			float NonSpatializedRadiusEnd = FMath::Min(WaveInstance->NonSpatializedRadiusStart, WaveInstance->NonSpatializedRadiusEnd);

			if (Params.Distance > 0.0f)
			{
				// If the user specified a distance below which to be fully 2D
				if (NonSpatializedRadiusEnd > 0.0f)
				{
					// We're in the non-spatialized domain
					if (Params.Distance < WaveInstance->NonSpatializedRadiusStart)
					{
						float NonSpatializationRange = WaveInstance->NonSpatializedRadiusStart - NonSpatializedRadiusEnd;
						NonSpatializationRange = FMath::Max(NonSpatializationRange, 1.0f);
						Params.NonSpatializedAmount = FMath::Clamp((WaveInstance->NonSpatializedRadiusStart - Params.Distance)/ NonSpatializationRange, 0.0f, 1.0f);
					}					
				}
				else
				{
					// Initialize to full omni-directionality (bigger value, more omni)
					static const float MaxNormalizedRadius = 1000000.0f;
					float NormalizedOmniRadus = FMath::Clamp(WaveInstance->NonSpatializedRadiusStart / Params.Distance, 0.0f, MaxNormalizedRadius);
					if (NormalizedOmniRadus > 1.0f)
					{
						float NormalizedOmniRadusSquared = NormalizedOmniRadus * NormalizedOmniRadus;
						Params.NonSpatializedAmount = 1.0f - 1.0f / NormalizedOmniRadusSquared;
					}
					else
					{
						Params.NonSpatializedAmount = 0.0f;
					}
				}

				//UE_LOGF(LogTemp, Log, "Distance: %.2f, NonSpatializedRadiusStart: %.2f, NonSpatializedRadiusEnd: %.2f, NonSpatializedAmount: %.2f", Params.Distance, WaveInstance->NonSpatializedRadiusStart, WaveInstance->NonSpatializedRadiusEnd, Params.NonSpatializedAmount);

			}
		}
		else
		{
			Params.NonSpatializedAmount = 0.0f;
		}

		Params.EmitterPosition = EmitterPosition;

		if (GetNumChannels() == 2)
		{
			Params.LeftChannelPosition = AudioDevice->GetListenerTransformedDirection(LeftChannelSourceLocation, nullptr);
			Params.RightChannelPosition = AudioDevice->GetListenerTransformedDirection(RightChannelSourceLocation, nullptr);			

		}
	}
	else
	{
		Params.NormalizedOmniRadius = 0.0f;
		Params.NonSpatializedAmount = 0.0f;
		Params.Distance = 0.0f;
		Params.EmitterPosition = FVector::ZeroVector;
	}
	Params.EmitterWorldPosition = WaveInstance->Location;

	int32 ListenerIndex = 0;
	if (WaveInstance->ActiveSound != nullptr)
	{
		Params.EmitterWorldRotation = WaveInstance->ActiveSound->Transform.GetRotation();
		ListenerIndex = WaveInstance->ActiveSound->GetClosestListenerIndex();
	}
	else
	{
		Params.EmitterWorldRotation = FQuat::Identity;
	}

	// Pass the actual listener orientation and position
	FTransform ListenerTransform;
	AudioDevice->GetListenerTransform(ListenerIndex, ListenerTransform);
	Params.ListenerOrientation = ListenerTransform.GetRotation();
	Params.ListenerPosition = ListenerTransform.GetLocation();

	return Params;
}

void FSoundSource::InitCommon()
{
	PlaybackTime = 0.0f;
	TickCount = 0;

	// Reset pause state
	bIsPausedByGame = false;
	bIsManuallyPaused = false;
	
#if ENABLE_AUDIO_DEBUG
	DebugInfo = MakeShared<FDebugInfo, ESPMode::ThreadSafe>();
#endif //ENABLE_AUDIO_DEBUG
}

void FSoundSource::UpdateCommon()
{
	check(WaveInstance);

	Pitch = WaveInstance->GetPitch();

	// Don't apply global pitch scale to UI sounds
	if (!WaveInstance->bIsUISound)
	{
		Pitch *= AudioDevice->GetGlobalPitchScale().GetValue();
	}

	Pitch = AudioDevice->ClampPitch(Pitch);

	// Track playback time even if the voice is not virtual, it can flip to being virtual while playing.
	const float DeviceDeltaTime = AudioDevice->GetDeviceDeltaTime();

	// Scale the playback time based on the pitch of the sound
	PlaybackTime += DeviceDeltaTime * Pitch;
}

float FSoundSource::GetPlaybackPercent() const
{
	const float Percentage = PlaybackTime / WaveInstance->WaveData->GetDuration();
	if (WaveInstance->LoopingMode == LOOP_Never)
	{
		return FMath::Clamp(Percentage, 0.0f, 1.0f);
	}
	else
	{
		// Wrap the playback percent for looping sounds
		return FMath::Fmod(Percentage, 1.0f);
	}

}

float FSoundSource::GetSourceSampleRate() const
{
	if (WaveInstance == nullptr || WaveInstance->WaveData == nullptr || WaveInstance->WaveData->bIsSourceBus)
	{
		return AudioDevice->GetSampleRate();
	}

	return WaveInstance->WaveData->GetSampleRateForCurrentPlatform();
}

int64 FSoundSource::GetNumFramesPlayed() const
{
	return NumFramesPlayed;
}

int32 FSoundSource::GetNumTotalFrames() const
{
	return NumTotalFrames;
}

int32 FSoundSource::GetStartFrame() const
{
	return StartFrame;
}

void FSoundSource::GetChannelLocations(FVector& Left, FVector&Right) const
{
	Left = LeftChannelSourceLocation;
	Right = RightChannelSourceLocation;
}


void FSoundSource::NotifyPlaybackData()
{
	const uint64 AudioComponentID = WaveInstance->ActiveSound->GetAudioComponentID();
	if (AudioComponentID > 0)
	{
		const USoundWave* SoundWave = WaveInstance->WaveData;

		if (WaveInstance->ActiveSound->bUpdatePlayPercentage)
		{
			const float PlaybackPercent = GetPlaybackPercent();
			FAudioThread::RunCommandOnGameThread([AudioComponentID, SoundWave, PlaybackPercent]()
			{
				if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
				{
					if (AudioComponent->OnAudioPlaybackPercent.IsBound())
					{
						AudioComponent->OnAudioPlaybackPercent.Broadcast(SoundWave, PlaybackPercent);
					}

					if (AudioComponent->OnAudioPlaybackPercentNative.IsBound())
					{
						AudioComponent->OnAudioPlaybackPercentNative.Broadcast(AudioComponent, SoundWave, PlaybackPercent);
					}
				}
			});
		}

		if (WaveInstance->ActiveSound->bUpdateSingleEnvelopeValue)
		{
			const float EnvelopeValue = GetEnvelopeValue();
			FAudioThread::RunCommandOnGameThread([AudioComponentID, SoundWave, EnvelopeValue]()
			{
				if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
				{
					if (AudioComponent->OnAudioSingleEnvelopeValue.IsBound())
					{
						AudioComponent->OnAudioSingleEnvelopeValue.Broadcast(SoundWave, EnvelopeValue);
					}

					if (AudioComponent->OnAudioSingleEnvelopeValueNative.IsBound())
					{
						AudioComponent->OnAudioSingleEnvelopeValueNative.Broadcast(AudioComponent, SoundWave, EnvelopeValue);
					}
				}
			});
		}

		// We do a broadcast from the active sound in this case, just update the envelope value of the wave instance here
		if (WaveInstance->ActiveSound->bUpdateMultiEnvelopeValue)
		{
			const float EnvelopeValue = GetEnvelopeValue();
			WaveInstance->SetEnvelopeValue(EnvelopeValue);
		}
	}
}

/*-----------------------------------------------------------------------------
	FNotifyBufferFinishedHooks implementation.
-----------------------------------------------------------------------------*/

void FNotifyBufferFinishedHooks::AddNotify(USoundNode* NotifyNode, UPTRINT WaveInstanceHash)
{
	Notifies.Add(FNotifyBufferDetails(NotifyNode, WaveInstanceHash));
}

UPTRINT FNotifyBufferFinishedHooks::GetHashForNode(USoundNode* NotifyNode) const
{
	for (const FNotifyBufferDetails& NotifyDetails : Notifies)
	{
		if (NotifyDetails.NotifyNode == NotifyNode)
		{
			return NotifyDetails.NotifyNodeWaveInstanceHash;
		}
	}

	return 0;
}

void FNotifyBufferFinishedHooks::DispatchNotifies(FWaveInstance* WaveInstance, const bool bStopped)
{
	for (int32 NotifyIndex = Notifies.Num() - 1; NotifyIndex >= 0; --NotifyIndex)
	{
		// All nodes get an opportunity to handle the notify if we're forcefully stopping the sound
		if (Notifies[NotifyIndex].NotifyNode)
		{
			if (Notifies[NotifyIndex].NotifyNode->NotifyWaveInstanceFinished(WaveInstance) && !bStopped)
			{
				break;
			}
		}
	}

}

void FNotifyBufferFinishedHooks::AddReferencedObjects( FReferenceCollector& Collector )
{
	for (FNotifyBufferDetails& NotifyDetails : Notifies)
	{
		Collector.AddReferencedObject( NotifyDetails.NotifyNode );
	}
}

FArchive& operator<<( FArchive& Ar, FNotifyBufferFinishedHooks& NotifyHook )
{
	if( !Ar.IsLoading() && !Ar.IsSaving() )
	{
		for (FNotifyBufferFinishedHooks::FNotifyBufferDetails& NotifyDetails : NotifyHook.Notifies)
		{
			Ar << NotifyDetails.NotifyNode;
		}
	}
	return( Ar );
}


/*-----------------------------------------------------------------------------
	FWaveInstance implementation.
-----------------------------------------------------------------------------*/

/** Helper to create good play order for FWaveInstance instances */
uint32 FWaveInstance::PlayOrderCounter = 0;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FWaveInstance::FWaveInstance(FWaveInstance&&) = default;
FWaveInstance& FWaveInstance::operator=(FWaveInstance&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * Constructor, initializing all member variables.
 *
 * @param InActiveSound		ActiveSound this wave instance belongs to.
 */
FWaveInstance::FWaveInstance(const UPTRINT InWaveInstanceHash, FActiveSound& InActiveSound)
	: WaveData(nullptr)
	, SoundClass(nullptr)
	, SoundSubmix(nullptr)
	, SourceEffectChain(nullptr)
	, ActiveSound(&InActiveSound)
	, Volume(0.0f)
	, DistanceAttenuation(1.0f)
	, OcclusionAttenuation(1.0f)
	, VolumeMultiplier(1.0f)
	, EnvelopValue(0.0f)
	, RelativeRenderCost(1.0f)
	, EnvelopeFollowerAttackTime(10)
	, EnvelopeFollowerReleaseTime(100)
	, Priority(1.0f)
	, VoiceCenterChannelVolume(0.0f)
	, RadioFilterVolume(0.0f)
	, RadioFilterVolumeThreshold(0.0f)
	, LFEBleed(0.0f)
	, LoopingMode(LOOP_Never)
	, StartTime(-1.f)
	, bEnableBusSends(ActiveSound->bEnableBusSendRoutingOverride)
	, bEnableBaseSubmix(ActiveSound->bEnableMainSubmixOutputOverride)
	, bEnableSubmixSends(ActiveSound->bEnableSubmixSendRoutingOverride)
	, bApplyRadioFilter(false)
	, bIsStarted(false)
	, bIsFinished(false)
	, bAlreadyNotifiedHook(false)
	, bUseSpatialization(false)
	, bEnableLowPassFilter(false)
	, bEnableHighPassFilter(false)
	, bIsOccluded(false)
	, bIsUISound(false)
	, bIsMusic(false)
	, bReverb(true)
	, bCenterChannelOnly(false)
	, bIsPaused(false)
	, bReportedSpatializationWarning(false)
	, bIsAmbisonics(false)
	, bIsStopping(false)
	, bIsDynamic(false)
	, SpatializationMethod(ESoundSpatializationAlgorithm::SPATIALIZATION_Default)
	, SpatializationPluginSettings(nullptr)
	, OcclusionPluginSettings(nullptr)
	, ReverbPluginSettings(nullptr)
	, SourceDataOverridePluginSettings(nullptr)
	, OutputTarget(EAudioOutputTarget::Speaker)
	, LowPassFilterFrequency(MAX_FILTER_FREQUENCY)
	, HighPassFilterFrequency(MIN_FILTER_FREQUENCY)
	, SoundClassFilterFrequency(MAX_FILTER_FREQUENCY)
	, OcclusionFilterFrequency(MAX_FILTER_FREQUENCY)
	, AmbientZoneFilterFrequency(MAX_FILTER_FREQUENCY)
	, AttenuationLowpassFilterFrequency(MAX_FILTER_FREQUENCY)
	, AttenuationHighpassFilterFrequency(MIN_FILTER_FREQUENCY)
	, Pitch(0.0f)
	, Location(FVector::ZeroVector)
	, NonSpatializedRadiusStart(0.0f)
	, NonSpatializedRadiusEnd(0.0f)
	, NonSpatializedRadiusMode(ENonSpatializedRadiusSpeakerMapMode::OmniDirectional)
	, StereoSpread(0.0f)
	, AttenuationDistance(0.0f)
	, ListenerToSoundDistance(0.0f)
	, ListenerToSoundDistanceForPanning(0.0f)
	, AbsoluteAzimuth(0.0f)
	, PlaybackTime(0.0f)
	, ReverbSendLevel(0.0f)
	, ManualReverbSendLevel(0.0f)
	, PlayOrder(0)
	, WaveInstanceHash(InWaveInstanceHash)
	, UserIndex(0)
{
	PlayOrder = ++PlayOrderCounter;
}

bool FWaveInstance::IsPlaying() const
{
	check(ActiveSound);

	if (!WaveData)
	{
		return false;
	}

	if (ActiveSound->bIsPreviewSound)
	{
		return true;
	}

	// TODO: move out of audio.  Subtitle system should be separate and just set VirtualizationMode to PlayWhenSilent
	const bool bHasSubtitles = ActiveSound->bHandleSubtitles && (ActiveSound->bHasExternalSubtitles || WaveData->Subtitles.Num() > 0);
	if (bHasSubtitles)
	{
		return true;
	}

	if (ActiveSound->IsPlayWhenSilent() && (!BypassPlayWhenSilentCVar || WaveData->bProcedural))
	{
		return true;
	}

	const float WaveInstanceVolume = Volume * VolumeMultiplier * GetDistanceAndOcclusionAttenuation() * GetDynamicVolume();
	if (WaveInstanceVolume > WaveInstanceMinVolumeThresholdCVar)
	{
		return true;
	}

	if (ActiveSound->ComponentVolumeFader.IsFadingIn())
	{
		return true;
	}

	return false;
}

/**
 * Notifies the wave instance that it has finished.
 */
void FWaveInstance::NotifyFinished( const bool bStopped )
{
	if( !bAlreadyNotifiedHook )
	{
		// Can't have a source finishing that hasn't started
		if( !bIsStarted )
		{
			UE_LOGF(LogAudio, Warning, "Received finished notification from waveinstance that hasn't started!" );
		}

		// We are finished.
		bIsFinished = true;

		// Avoid double notifications.
		bAlreadyNotifiedHook = true;

		NotifyBufferFinishedHooks.DispatchNotifies(this, bStopped);
	}
}

/**
 * Stops the wave instance without notifying NotifyWaveInstanceFinishedHook. This will NOT stop wave instance
 * if it is set up to loop indefinitely or set to remain active.
 */
void FWaveInstance::StopWithoutNotification( void )
{
	if( LoopingMode == LOOP_Forever || ActiveSound->bShouldRemainActiveIfDropped )
	{
		// We don't finish if we're either indefinitely looping or the audio component explicitly mandates that we should
		// remain active which is e.g. used for engine sounds and such.
		bIsFinished = false;
	}
	else
	{
		// We're finished.
		bIsFinished = true;
	}
}

FArchive& operator<<( FArchive& Ar, FWaveInstance* WaveInstance )
{
	if( !Ar.IsLoading() && !Ar.IsSaving() )
	{
		Ar << WaveInstance->WaveData;
		Ar << WaveInstance->SoundClass;
		Ar << WaveInstance->NotifyBufferFinishedHooks;
	}
	return( Ar );
}

void FWaveInstance::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( WaveData );

	if (USynthSound* SynthSound = Cast<USynthSound>(WaveData))
	{
		Collector.AddReferencedObject(SynthSound->GetOwningSynthComponentPtr());
	}

	auto AddSubmixSendRef = [&Collector](FSoundSubmixSendInfoBase& Info)
	{
		if (Info.SoundSubmix)
		{
			Collector.AddReferencedObject(Info.SoundSubmix);
		}	
	};

	Algo::ForEach(SoundSubmixSends, AddSubmixSendRef);
	Algo::ForEach(AttenuationSubmixSends, AddSubmixSendRef);
	
	Collector.AddReferencedObject( SoundClass );
	NotifyBufferFinishedHooks.AddReferencedObjects( Collector );

	auto AddBusSendRef = [&Collector](FSoundSourceBusSendInfo& Info)
		{
			if (Info.SoundSourceBus)
			{
				Collector.AddReferencedObject(Info.SoundSourceBus);
			}
			if (Info.AudioBus)
			{
				Collector.AddReferencedObject(Info.AudioBus);
			}
		};

	for (int32 BusSendCount = 0; BusSendCount < (int32)EBusSendType::Count; ++BusSendCount)
	{
		Algo::ForEach(BusSends[BusSendCount], AddBusSendRef);
	}
}

float FWaveInstance::GetActualVolume() const
{
	// Include all volumes
	float ActualVolume = GetVolume() * GetDistanceAndOcclusionAttenuation();
	if (ActualVolume != 0.0f)
	{
		ActualVolume *= GetDynamicVolume();

		check(ActiveSound);
		if (!ActiveSound->bIsPreviewSound)
		{
			check(ActiveSound->AudioDevice);
			ActualVolume *= ActiveSound->AudioDevice->GetPrimaryVolume();
		}
	}

	return ActualVolume;
}

float FWaveInstance::GetDistanceAndOcclusionAttenuation() const
{
	return DistanceAttenuation * OcclusionAttenuation;
}

float FWaveInstance::GetDistanceAttenuation() const
{
	return DistanceAttenuation;
}

float FWaveInstance::GetOcclusionAttenuation() const
{
	return OcclusionAttenuation;
}

float FWaveInstance::GetDynamicVolume() const
{
	float OutVolume = 1.0f;

	if (GEngine)
	{
		if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
		{
			if (WaveData)
			{
				OutVolume *= DeviceManager->GetDynamicSoundVolume(ESoundType::Wave, WaveData->GetFName());
			}

			if (ActiveSound)
			{
				if (const USoundCue* Sound = Cast<USoundCue>(ActiveSound->GetSound()))
				{
					OutVolume *= DeviceManager->GetDynamicSoundVolume(ESoundType::Cue, Sound->GetFName());
				}
			}

			if (SoundClass)
			{
				OutVolume *= DeviceManager->GetDynamicSoundVolume(ESoundType::Class, SoundClass->GetFName());
			}
		}
	}

	return OutVolume;
}

float FWaveInstance::GetVolumeWithDistanceAndOcclusionAttenuation() const
{
	return GetVolume() * GetDistanceAndOcclusionAttenuation();
}

float FWaveInstance::GetPitch() const
{
	return Pitch;
}

float FWaveInstance::GetVolume() const
{
	// Only includes non-attenuation and non-app volumes
	return Volume * VolumeMultiplier;
}

bool FWaveInstance::ShouldStopDueToMaxConcurrency() const
{
	check(ActiveSound);
	return ActiveSound->bShouldStopDueToMaxConcurrency;
}

float FWaveInstance::GetVolumeWeightedPriority() const
{
	// If priority has been set via bAlwaysPlay, it will have a priority larger than MAX_SOUND_PRIORITY. If that's the case, we should ignore volume weighting.
	if (Priority > MAX_SOUND_PRIORITY)
	{
		return Priority;
	}

	// This will result in zero-volume sounds still able to be sorted due to priority but give non-zero volumes higher priority than 0 volumes
	float ActualVolume = GetVolumeWithDistanceAndOcclusionAttenuation();
	if (ActualVolume > 0.0f)
	{
		// Only check for bypass if the actual volume is greater than 0.0
		if (WaveData && WaveData->bBypassVolumeScaleForPriority)
		{
			return Priority;
		}
		else
		{
			return ActualVolume * Priority;
		}
	}
	else if (IsStopping())
	{
		// Stopping sounds will be sorted above 0-volume sounds
		return ActualVolume * Priority - MAX_SOUND_PRIORITY - 1.0f;
	}
	else
	{
		return Priority - 2.0f * MAX_SOUND_PRIORITY - 1.0f;
	}
}

bool FWaveInstance::IsSeekable() const
{
	check(WaveData);

	if (StartTime == 0.0f)
	{
		return false;
	}

	return WaveData->IsSeekable();
}

bool FWaveInstance::IsStreaming() const
{
	return FPlatformProperties::SupportsAudioStreaming() && WaveData != nullptr && WaveData->IsStreaming(nullptr);
}

bool FWaveInstance::GetUseSpatialization() const
{
	return AllowAudioSpatializationCVar && bUseSpatialization;
}

FString FWaveInstance::GetName() const
{
	if (WaveData)
	{
		return WaveData->GetName();
	}
	return TEXT("Null");
}



#define UE_MAKEFOURCC(ch0, ch1, ch2, ch3)\
	((uint32)(uint8)(ch0) | ((uint32)(uint8)(ch1) << 8) |\
	((uint32)(uint8)(ch2) << 16) | ((uint32)(uint8)(ch3) << 24 ))

#define UE_mmioFOURCC(ch0, ch1, ch2, ch3)\
	UE_MAKEFOURCC(ch0, ch1, ch2, ch3)

#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(push, 2)
#endif

// Main Riff-Wave header.
struct FRiffWaveHeaderChunk
{
	uint32	rID;			// Contains 'RIFF'
	uint32	ChunkLen;		// Remaining length of the entire riff chunk (= file).
	uint32	wID;			// Form type. Contains 'WAVE' for .wav files.
};

// General chunk header format.
struct FRiffChunkOld
{
	uint32	ChunkID;		  // General data chunk ID like 'data', or 'fmt '
	uint32	ChunkLen;		  // Length of the rest of this chunk in bytes.
};

// ChunkID: 'fmt ' ("WaveFormatEx" structure )
struct FRiffFormatChunk
{
	uint16   wFormatTag;        // Format type: 1 = PCM
	uint16   nChannels;         // Number of channels (i.e. mono, stereo...).
	uint32   nSamplesPerSec;    // Sample rate. 44100 or 22050 or 11025  Hz.
	uint32   nAvgBytesPerSec;   // For buffer estimation  = sample rate * BlockAlign.
	uint16   nBlockAlign;       // Block size of data = Channels times BYTES per sample.
	uint16   wBitsPerSample;    // Number of bits per sample of mono data.
	uint16   cbSize;            // The count in bytes of the size of extra information (after cbSize).
};

// ChunkID: 'cue ' 
// A cue chunk specifies one or more sample offsets which are often used to mark noteworthy sections of audio. For example, 
// the beginning and end of a verse in a song may have cue points to make them easier to find. The cue chunk is optional and 
// if included, a single cue chunk should specify all cue points for the "WAVE" chunk. 
// No more than one cue chunk is allowed in a "WAVE" chunk.
struct FRiffCueChunk
{
	uint32 ChunkID;			// 'cue '
	uint32 ChunkDataSize;	// Depends on the number of cue points
	uint32 NumCuePoints;	// Number of cue points in the list
};

struct FRiffCuePointChunk
{
	uint32 CueID;			// Unique ID value for the cue point
	uint32 Position;		// Play order position
	uint32 DataChunkID;		// RIFF ID of corresponding data chunk
	uint32 ChunkStart;		// Byte offset of data chunk
	uint32 BlockStart;		// Byte offset of sample of first channel
	uint32 SampleOffset;	// Byte offset to sample byte of first channel
};

// ChunkID: 'smpl'
// The sample chunk allows a MIDI sampler to use the Wave file as a collection of samples.
// The 'smpl' chunk is optional and if included, a single 'smple' chunk should specify all Sample Loops
struct FRiffSampleChunk
{
	uint32 ChunkID;				// 'smpl'
	uint32 ChunkDataSize;		// Depends on the number of sample loops
	uint32 ManufacturerCode;	// The MIDI Manufacturers Association manufacturer code
	uint32 Product;				// The Product / Model ID of the target device, specific to the manufacturer
	uint32 SamplePeriod;		// The period of one sample in nanoseconds.
	uint32 MidiUnityNote;		// The MIDI note that will play when this sample is played at its current pitch
	uint32 MidiPitchFraction;	// The fraction of a semitone up from the specified note. 
	uint32 SmpteFormat;			// The SMPTE format. Possible values: 0, 24, 25, 29, 30
	uint32 SmpteOffset;			// Specifies a time offset for the sample, if the sample should start at a later time and not immediately.
	uint32 NumSampleLoops;		// Number of sample loops contained in this chunks data
	uint32 NumSampleDataBytes;	// Number of bytes of optional sampler specific data that follows the sample loops. zero if there is no such data.
};

struct FRiffSampleLoopChunk
{
	uint32 LoopID;			// A unique ID of the loop, which could be a cue point
	uint32 LoopType;		// The loop type. 0: Forward Looping, 1: Ping-Pong, 2: Backward, 3-31: future standard types. >=32: manufacturer specific types
	uint32 StartFrame;		// Start point of the loop in samples
	uint32 EndFrame;		// End point of the loop in samples. The end sample is also played.
	uint32 Fraction;		// The resolution at which this loop should be fine tuned.
	uint32 NumPlayTimes;	// The number of times to play the loop. A value of zero means inifity. In a Midi sampler, that may mean infinite sustain.
};

struct FRiffListChunk
{
	uint32 ChunkID;			// 'list'
	uint32 ChunkDataSize;	// Depends on contained text
	uint32 TypeID;			// always 'adtl'
};

struct FRiffLabelChunk
{
	uint32 ChunkID;			// 'labl'
	uint32 ChunkDataSize;	// depends on contained text
	uint32 CuePointID;		// Cue Point ID associated with the label
};

struct FRiffNoteChunk
{
	uint32 ChunkID;			// 'note'
	uint32 ChunkDataSize;	// Depends on size of contained text
	uint32 CuePointID;		// ID associated with the note
};

struct FRiffLabeledTextChunk
{
	uint32 ChunkID;			// 'ltxt'
	uint32 ChunkDataSize;	// Depends on contained text
	uint32 CuePointID;		// ID associated with the labeled text
	uint32 SampleLength;	// Defines how many samples from the cue point the region or section spans
	uint32 PurposeID;		// Unused: Specifies what the text is used for. 'scrp' means script text. 'capt' means close-caption.
	uint16 Country;			// Unused 
	uint16 Language;		// Unused
	uint16 Dialect;			// Unused 
	uint16 CodePage;		// Unused
};

// Specification of the Broadcast Wave Format(BWF)
// https://tech.ebu.ch/docs/tech/tech3285.pdf
struct FRiffBroadcastAudioExtension 
{
	uint32 ChunkID;					// 'bext'
	uint32 ChunkDataSize;			// Depends on contained text
	uint8 Description[256];			// ASCII : «Description of the sound sequence» 
	uint8 Originator[32];			// ASCII : «Name of the originator» 
	uint8 OriginatorReference[32];	// ASCII : «Reference of the originator» 
	uint8 OriginationDate[10];		// ASCII : «yyyy:mm:dd» 
	uint8 OriginationTime[8];		// ASCII : «hh:mm:ss» 
	uint32 TimeReferenceLow;		// First sample count since midnight, low word 
	uint32 TimeReferenceHigh;		// First sample count since midnight, high word	

	uint16 Version;					// Version of the BWF; unsigned binary number 
	uint8 UMID[64];					// Binary SMPTE UMID 
	uint16 LoudnessValue;			// WORD : «Integrated Loudness Value of the filein LUFS (multiplied by 100) » 
	uint16 LoudnessRange;			// WORD : «Loudness Range of the file in LU (multiplied by 100) » 
	uint16 MaxTruePeakLevel;		// WORD : «Maximum True Peak Level of the file expressed as dBTP (multiplied by 100) » 
	uint16 MaxMomentaryLoudness;	// WORD : «Highest value of the Momentary Loudness Level of the file in LUFS (multiplied by 100) » 
	uint16 MaxShortTermLoudness;	// WORD : «Highest value of the Short-TermLoudness Level of the file in LUFS (multiplied by 100) » 
	uint8 Reserved[180];			// 180 bytes, reserved for future use, set to “NULL” 
	uint8 CodingHistory[1];			// ASCII : « History coding » (truncated here are don't care about it) 
};

struct FRiffIXmlChunk
{
	uint32 ChunkID;					// 'iXML'
	uint32 ChunkDataSize;			// Depends on contained text
	uint8 XmlText[1];				// Raw XML blob. (truncated as the size is based on the chunk size).
};	

// FExtendedFormatChunk subformat GUID.
struct FSubformatGUID
{
	uint32 Data1;				// Format type, corresponds to a wFormatTag in WaveFormatEx.

								// Fixed values for all extended wave formats.
	uint16 Data2 = 0x0000;
	uint16 Data3 = 0x0010;
	uint8 Data4[8];

	FSubformatGUID()
	{
		Data4[0] = 0x80;
		Data4[1] = 0x00;
		Data4[2] = 0x00;
		Data4[3] = 0xaa;
		Data4[4] = 0x00;
		Data4[5] = 0x38;
		Data4[6] = 0x9b;
		Data4[7] = 0x71;
	}
};

// ChunkID: 'fmt ' ("WaveFormatExtensible" structure)
struct FExtendedFormatChunk
{
	FRiffFormatChunk Format;			// Standard WaveFormatEx ('fmt ') chunk, with
									// wFormatTag == WAVE_FORMAT_EXTENSIBLE and cbSize == 22
	union
	{
		uint16 wValidBitsPerSample;	// Actual bits of precision. Can be less than wBitsPerSample.
		uint16 wSamplesPerBlock;	// Valid if wValidBitsPerSample == 0. Used by compressed formats.
		uint16 wReserved;			// If neither applies, set to 0.
	} Samples;
	uint32 dwChannelMask;			// Which channels are present in the stream.
	FSubformatGUID SubFormat;		// Subformat identifier.
};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(pop)
#endif

const TArray<uint32>& FWaveModInfo::GetRequiredWaveChunkIds()
{
	static TArray<uint32> RequiredChunkIds =
	{
		UE_mmioFOURCC('f', 'm', 't', ' '),
		UE_mmioFOURCC('d', 'a', 't', 'a')
	};
	
	return RequiredChunkIds;
}

const TArray<uint32>& FWaveModInfo::GetOptionalWaveChunkIds()
{
	static TArray<uint32> OptionalChunkIds =
	{
		UE_mmioFOURCC('f', 'a', 'c', 't'),
		UE_mmioFOURCC('c', 'u', 'e', ' '),
		UE_mmioFOURCC('p', 'l', 's', 't'),
		UE_mmioFOURCC('L', 'I', 'S', 'T'),
		UE_mmioFOURCC('l', 'a', 'b', 'l'),
		UE_mmioFOURCC('l', 't', 'x', 't'),
		UE_mmioFOURCC('n', 'o', 't', 'e'),
		UE_mmioFOURCC('s', 'm', 'p', 'l'),
		UE_mmioFOURCC('i', 'n', 's', 't'),
		UE_mmioFOURCC('a', 'c', 'i', 'd'),
		UE_mmioFOURCC('b', 'e', 'x', 't'),
		UE_mmioFOURCC('i', 'X', 'M', 'L')
	};

	return OptionalChunkIds;
}

const uint32 FWaveModInfo::GetChunkId(const char ChunkID[4])
{
	return UE_mmioFOURCC(ChunkID[0], ChunkID[1], ChunkID[2], ChunkID[3]);
}

bool IsKnownChunkId(uint32 ChunkId)
{
	bool bIsKnown = FWaveModInfo::GetRequiredWaveChunkIds().Contains(ChunkId) ||
					FWaveModInfo::GetOptionalWaveChunkIds().Contains(ChunkId);

	return bIsKnown;
}

FRiffChunkOld* FindRiffChunk(FRiffChunkOld* RiffChunkStart, const uint8* RiffChunkEnd, uint32 ChunkId)
{
	// invalid RiffChunkStart
	if (!ensure(RiffChunkStart))
	{
		return nullptr;
	}

	// invalid RiffChunkEnd
	if (!ensure(RiffChunkEnd))
	{
		return nullptr;
	}


	FRiffChunkOld* RiffChunk = RiffChunkStart;


	while ((uint8*)RiffChunk < RiffChunkEnd)
	{
		if (INTEL_ORDER32(RiffChunk->ChunkID) == ChunkId)
		{
			return RiffChunk;
		}

		// If the file has non standard chunks identifiers AND is not padding then we cannot be sure what sort of values 
		// RiffChunk can contain as we will be reading garbage and potentially 'ChunkLen' could be zero which would lead 
		// to an infinite loop, so check for that here.
		if (RiffChunk->ChunkLen == 0)
		{
			return nullptr;
		}

		// The format specifies that chunks should be word aligned however some content creation tools seem to ignore this. 
		// If ChunkLen is even then we can just advance to the next chunk. If it is odd then we need to try and determine if
		// this file obeys the padding rule.
		if (INTEL_ORDER32(RiffChunk->ChunkLen) % 2 == 0)
		{
			RiffChunk = (FRiffChunkOld*)((uint8*)RiffChunk + INTEL_ORDER32(RiffChunk->ChunkLen) + 8);
		}
		else
		{
			// First we find the next chunk if the file is not obeying the padding rule, note that we must check to make sure
			// that the chunk is still within the bounds of our memory as at this point we can no longer trust the data.
			// If the next chunk (NoPaddingChunk) has an identifier that we recognize then we assume that the current chunk
			// (RiffChunk) was not padded. If we do not find a chunk id that we recognize we should assume that padding is
			// required and try that instead.
			FRiffChunkOld* NoPaddingChunk = (FRiffChunkOld*)((uint8*)RiffChunk + INTEL_ORDER32(RiffChunk->ChunkLen) + 8);
			if ((uint8*)NoPaddingChunk + 8 < RiffChunkEnd && IsKnownChunkId(NoPaddingChunk->ChunkID))
			{
				RiffChunk = NoPaddingChunk;
			}
			else
			{
				RiffChunk = (FRiffChunkOld*)((uint8*)RiffChunk + FWaveModInfo::Pad16Bit(INTEL_ORDER32(RiffChunk->ChunkLen)) + 8);
			}
		}
	}

	return nullptr;
}

//
//	Figure out the WAVE file layout.
//
bool FWaveModInfo::ReadWaveInfo( const uint8* WaveData, int32 WaveDataSize, FString* ErrorReason, bool InHeaderDataOnly, void** OutFormatHeader)
{
	FRiffFormatChunk* FmtChunk;
	FExtendedFormatChunk* FmtChunkEx = nullptr;
	FRiffWaveHeaderChunk* RiffHdr = (FRiffWaveHeaderChunk*)WaveData;
	WaveDataEnd = WaveData + WaveDataSize;

	if (WaveDataSize == 0)
	{
		return false;
	}

	// Verify we've got a real 'WAVE' header.
#if PLATFORM_LITTLE_ENDIAN
	if (RiffHdr->wID != UE_mmioFOURCC('W','A','V','E'))
	{
		if (ErrorReason) *ErrorReason = TEXT("Invalid WAVE file.");
		return false;
	}
#else
	if ((RiffHdr->wID != (UE_mmioFOURCC('W','A','V','E'))) &&
	     (RiffHdr->wID != (UE_mmioFOURCC('E','V','A','W'))))
	{
		ErrorReason = TEXT("Invalid WAVE file.")
			return false;
	}

	bool AlreadySwapped = (RiffHdr->wID == (UE_mmioFOURCC('W','A','V','E')));
	if (!AlreadySwapped)
	{
		RiffHdr->rID = INTEL_ORDER32(RiffHdr->rID);
		RiffHdr->ChunkLen = INTEL_ORDER32(RiffHdr->ChunkLen);
		RiffHdr->wID = INTEL_ORDER32(RiffHdr->wID);
	}
#endif

	FRiffChunkOld* RiffChunkStart = (FRiffChunkOld*)&WaveData[3 * 4];
	pMasterSize = &RiffHdr->ChunkLen;

	// Look for the 'fmt ' chunk.
	FRiffChunkOld* RiffChunk = FindRiffChunk(RiffChunkStart, WaveDataEnd, UE_mmioFOURCC('f', 'm', 't', ' '));

	if (RiffChunk == nullptr)
	{
		#if !PLATFORM_LITTLE_ENDIAN  // swap them back just in case.
			if( !AlreadySwapped )
			{
				RiffHdr->rID = INTEL_ORDER32( RiffHdr->rID );
				RiffHdr->ChunkLen = INTEL_ORDER32( RiffHdr->ChunkLen );
				RiffHdr->wID = INTEL_ORDER32( RiffHdr->wID );
			}
		#endif
			if (ErrorReason)
			{
				*ErrorReason = TEXT("Invalid WAVE file.");
			}

		return( false );
	}

	FmtChunk = (FRiffFormatChunk*)((uint8*)RiffChunk + 8);
#if !PLATFORM_LITTLE_ENDIAN
	if (!AlreadySwapped)
	{
		FmtChunk->wFormatTag = INTEL_ORDER16(FmtChunk->wFormatTag);
		FmtChunk->nChannels = INTEL_ORDER16(FmtChunk->nChannels);
		FmtChunk->nSamplesPerSec = INTEL_ORDER32(FmtChunk->nSamplesPerSec);
		FmtChunk->nAvgBytesPerSec = INTEL_ORDER32(FmtChunk->nAvgBytesPerSec);
		FmtChunk->nBlockAlign = INTEL_ORDER16(FmtChunk->nBlockAlign);
		FmtChunk->wBitsPerSample = INTEL_ORDER16(FmtChunk->wBitsPerSample);
	}
#endif
	pBitsPerSample = &FmtChunk->wBitsPerSample;
	pSamplesPerSec = &FmtChunk->nSamplesPerSec;
	pAvgBytesPerSec = &FmtChunk->nAvgBytesPerSec;
	pBlockAlign = &FmtChunk->nBlockAlign;
	pChannels = &FmtChunk->nChannels;
	pFormatTag = &FmtChunk->wFormatTag;

	if (OutFormatHeader != NULL)
	{
		*OutFormatHeader = FmtChunk;
	}

	// If we have an extended fmt chunk, the format tag won't be a wave format. Instead we need to read the subformat ID.
	if (INTEL_ORDER32(RiffChunk->ChunkLen) >= 40 && FmtChunk->wFormatTag == 0xFFFE) // WAVE_FORMAT_EXTENSIBLE
	{
		FmtChunkEx = (FExtendedFormatChunk*)((uint8*)RiffChunk + 8);

#if !PLATFORM_LITTLE_ENDIAN
		if (!AlreadySwapped)
		{
			FmtChunkEx->Samples.wValidBitsPerSample = INTEL_ORDER16(FmtChunkEx->Samples.wValidBitsPerSample);
			FmtChunkEx->SubFormat.Data1 = INTEL_ORDER32(FmtChunkEx->SubFormat.Data1);
			FmtChunkEx->SubFormat.Data2 = INTEL_ORDER16(FmtChunkEx->SubFormat.Data2);
			FmtChunkEx->SubFormat.Data3 = INTEL_ORDER16(FmtChunkEx->SubFormat.Data3);
			*((uint64*)FmtChunkEx->SubFormat.Data4) = INTEL_ORDER64(*(uint64*)FmtChunkEx->SubFormat.Data4);
		}
#endif

		bool bValid = true;
		static const FSubformatGUID GUID;

		if (FmtChunkEx->SubFormat.Data1 == 0x00000001 /* PCM */ &&
			FmtChunkEx->Samples.wValidBitsPerSample > 0 && FmtChunkEx->Samples.wValidBitsPerSample != FmtChunk->wBitsPerSample)
		{
			bValid = false;
			if (ErrorReason) *ErrorReason = TEXT("Unsupported WAVE file format: actual bit rate does not match the container size.");
		}
		else if (FMemory::Memcmp((uint8*)&FmtChunkEx->SubFormat + 4, (uint8*)&GUID + 4, sizeof(GUID) - 4) != 0)
		{
			bValid = false;
			if (ErrorReason) *ErrorReason = TEXT("Unsupported WAVE file format: subformat identifier not recognized.");
		}

		if (!bValid)
		{
#if !PLATFORM_LITTLE_ENDIAN // swap them back just in case.
			if (!AlreadySwapped)
			{
				FmtChunkEx->Samples.wValidBitsPerSample = INTEL_ORDER16(FmtChunkEx->Samples.wValidBitsPerSample);
				FmtChunkEx->SubFormat.Data1 = INTEL_ORDER32(FmtChunkEx->SubFormat.Data1);
				FmtChunkEx->SubFormat.Data2 = INTEL_ORDER16(FmtChunkEx->SubFormat.Data2);
				FmtChunkEx->SubFormat.Data3 = INTEL_ORDER16(FmtChunkEx->SubFormat.Data3);
				*((uint64*)FmtChunkEx->SubFormat.Data4) = INTEL_ORDER64(*(uint64*)FmtChunkEx->SubFormat.Data4);
			}
#endif
			return (false);
		}

		// Set the format tag pointer to the subformat GUID.
		pFormatTag = reinterpret_cast<uint16*>(&FmtChunkEx->SubFormat.Data1);
	}


	// Look for the 'data' chunk.
	RiffChunk = FindRiffChunk(RiffChunkStart, WaveDataEnd, UE_mmioFOURCC('d', 'a', 't', 'a'));

	if (RiffChunk == nullptr)
	{
		#if !PLATFORM_LITTLE_ENDIAN  // swap them back just in case.
			if (!AlreadySwapped)
			{
				RiffHdr->rID = INTEL_ORDER32(RiffHdr->rID);
				RiffHdr->ChunkLen = INTEL_ORDER32(RiffHdr->ChunkLen);
				RiffHdr->wID = INTEL_ORDER32(RiffHdr->wID);
				FmtChunk->wFormatTag = INTEL_ORDER16(FmtChunk->wFormatTag);
				FmtChunk->nChannels = INTEL_ORDER16(FmtChunk->nChannels);
				FmtChunk->nSamplesPerSec = INTEL_ORDER32(FmtChunk->nSamplesPerSec);
				FmtChunk->nAvgBytesPerSec = INTEL_ORDER32(FmtChunk->nAvgBytesPerSec);
				FmtChunk->nBlockAlign = INTEL_ORDER16(FmtChunk->nBlockAlign);
				FmtChunk->wBitsPerSample = INTEL_ORDER16(FmtChunk->wBitsPerSample);
				if (FmtChunkEx != nullptr)
				{
					FmtChunkEx->Samples.wValidBitsPerSample = INTEL_ORDER16(FmtChunkEx->Samples.wValidBitsPerSample);
					FmtChunkEx->SubFormat.Data1 = INTEL_ORDER32(FmtChunkEx->SubFormat.Data1);
					FmtChunkEx->SubFormat.Data2 = INTEL_ORDER16(FmtChunkEx->SubFormat.Data2);
					FmtChunkEx->SubFormat.Data3 = INTEL_ORDER16(FmtChunkEx->SubFormat.Data3);
					*((uint64*)FmtChunkEx->SubFormat.Data4) = INTEL_ORDER64(*(uint64*)FmtChunkEx->SubFormat.Data4);
				}
			}
		#endif
		if (ErrorReason) *ErrorReason = TEXT("Invalid WAVE file.");
		return false;
	}

#if !PLATFORM_LITTLE_ENDIAN  // swap them back just in case.
	if (AlreadySwapped) // swap back into Intel order for chunk search...
	{
		RiffChunk->ChunkLen = INTEL_ORDER32(RiffChunk->ChunkLen);
	}
#endif

	SampleDataStart = (uint8*)RiffChunk + 8;
	pWaveDataSize = &RiffChunk->ChunkLen;
	SampleDataSize = INTEL_ORDER32( RiffChunk->ChunkLen );
	SampleDataEnd = SampleDataStart + SampleDataSize;

#if !WITH_SNDFILE_IO
	if (!IsFormatSupported())
	{
		ReportImportFailure();
		if (ErrorReason) *ErrorReason = TEXT("Unsupported wave file format.  Only PCM, ADPCM, and DVI ADPCM can be imported.");
		return false;
	}
#endif //WITH_SNDFILE_IO

	if (!InHeaderDataOnly && IsFormatSupported())
	{
		if ((uint8*)SampleDataEnd > (uint8*)WaveDataEnd)
		{
			UE_LOG(LogAudio, Warning, TEXT("Wave data chunk exceeds end of wave file by %" SIZE_T_FMT " bytes, truncating"), (SampleDataEnd - WaveDataEnd));

			// Fix it up by clamping data chunk.
			SampleDataEnd = (uint8*)WaveDataEnd;
			SampleDataSize = SampleDataEnd - SampleDataStart;
			RiffChunk->ChunkLen = INTEL_ORDER32(SampleDataSize);
		}

		// Truncate any bad sample data, does not affect exported data
		// Corrupt data remains present in the raw wave data
		if (pBlockAlign && *pBlockAlign > 0)
		{
			const uint16 BlockAlign = *pBlockAlign; // NumChannels * BytesPerSample
			const uint32 ExtraBytes = SampleDataSize % BlockAlign;

			if (ExtraBytes > 0)
			{
				UE_LOGF(LogAudio, Warning, "Wave data chunk has %u invalid byte(s), truncating invalid block of compressed data", ExtraBytes);

				SampleDataEnd = SampleDataEnd - ExtraBytes;
				SampleDataSize = SampleDataEnd - SampleDataStart;
			}
		}

		NewDataSize = SampleDataSize;

		#if !PLATFORM_LITTLE_ENDIAN
		if (!AlreadySwapped)
		{
			if (FmtChunk->wBitsPerSample == 16)
			{
				for (uint16* i = (uint16*)SampleDataStart; i < (uint16*)SampleDataEnd; i++)
				{
					*i = INTEL_ORDER16(*i);
				}
			}
			else if (FmtChunk->wBitsPerSample == 32)
			{
				for (uint32* i = (uint32*)SampleDataStart; i < (uint32*)SampleDataEnd; i++)
				{
					*i = INTEL_ORDER32( *i );
				}
			}
		}
		#endif
	}

	// Couldn't byte swap this before, since it'd throw off the chunk search.
#if !PLATFORM_LITTLE_ENDIAN
	*pWaveDataSize = INTEL_ORDER32(*pWaveDataSize);
#endif

	// Look for the cue chunks
	RiffChunk = FindRiffChunk(RiffChunkStart, WaveDataEnd, UE_mmioFOURCC('c', 'u', 'e', ' '));

	// Cue chunks are optional
	if (RiffChunk != nullptr)
	{
		FRiffCueChunk* CueChunk = (FRiffCueChunk*)((uint8*)RiffChunk);

		// Guard against a crafted NumCuePoints that would trigger integer overflow in TArray allocation.
		// ChunkDataSize covers the NumCuePoints field (4 bytes) plus the cue point array entries.
		const uint8* CuePointDataStart = (uint8*)RiffChunk + sizeof(FRiffCueChunk);

		// Number of bytes for cue section reported by the wave file
		const uint32 NumCueDataBytes = CueChunk->ChunkDataSize >= sizeof(uint32) ? CueChunk->ChunkDataSize - sizeof(uint32) : 0;

		// Maximum number of cue points that could fit in the reported number of bytes.
		const uint32 MaxCuePointsByChunk = NumCueDataBytes / sizeof(FRiffCuePointChunk);

		// Maximum number of cue points that could fit before the end of the file
		const uint32 MaxCuePointsByFile = CuePointDataStart < (const uint8*)WaveDataEnd
			? static_cast<uint32>(((const uint8*)WaveDataEnd - CuePointDataStart) / sizeof(FRiffCuePointChunk))
			: 0;

		// Take the min of all these to make sure the reported number of cue points can't break out of the RIFF cue chunk or the file as a whole
		const uint32 SafeNumCuePoints = FMath::Min(FMath::Min(CueChunk->NumCuePoints, MaxCuePointsByChunk), MaxCuePointsByFile);

		if (SafeNumCuePoints != CueChunk->NumCuePoints)
		{
			UE_LOG(LogAudio, Warning, TEXT("FWaveModInfo::ReadWaveInfo: WAV cue chunk NumCuePoints (%u) exceeds available data; clamping to %u. File may be malformed."),
				CueChunk->NumCuePoints, SafeNumCuePoints);
		}

		WaveCues.Reset(SafeNumCuePoints);

		// Get to the first cue point chunk
		FRiffCuePointChunk* CuePointChunks = (FRiffCuePointChunk*)CuePointDataStart;
		for (uint32 CuePointId = 0; CuePointId < SafeNumCuePoints; ++CuePointId)
		{
			FWaveCue NewCue;
			NewCue.CuePointID = CuePointChunks[CuePointId].CueID;
			NewCue.Position = CuePointChunks[CuePointId].Position;
			WaveCues.Add(NewCue);
		
		}
		
		// Now look for label chunk for labels for cues
		// Look for the 'list' chunk.
		RiffChunk = FindRiffChunk(RiffChunkStart, WaveDataEnd, UE_mmioFOURCC('L', 'I', 'S', 'T'));

		// Label chunks are also optional
		if (RiffChunk != nullptr)
		{
			FRiffListChunk* ListChunk = (FRiffListChunk*)((uint8*)RiffChunk);
			
			RiffChunk = (FRiffChunkOld*)(ListChunk + 1);
			uint8* ListChunkEnd = (uint8*)RiffChunk + ListChunk->ChunkDataSize + 4;
			if (ListChunkEnd > WaveDataEnd)
			{
				ListChunkEnd = (uint8*)WaveDataEnd;
			}

			while (((uint8*)RiffChunk + 8) <= ListChunkEnd)
			{
				// Labeled Text Chunk
				// This information is often displayed in marked regions of a waveform in digital audio editors.
				if (INTEL_ORDER32(RiffChunk->ChunkID) == UE_mmioFOURCC('l', 't', 'x', 't'))
				{
					FRiffLabeledTextChunk* LabeledTextChunk = (FRiffLabeledTextChunk*)((uint8*)RiffChunk);
					for (FWaveCue& WaveCue : WaveCues)
					{
						if (WaveCue.CuePointID == LabeledTextChunk->CuePointID)
						{
							WaveCue.SampleLength = LabeledTextChunk->SampleLength;
							break;
						}
					}
				}
				// Labeled Chunk
				else if (INTEL_ORDER32(RiffChunk->ChunkID) == UE_mmioFOURCC('l', 'a', 'b', 'l'))
				{
					FRiffLabelChunk* LabelChunk = (FRiffLabelChunk*)((uint8*)RiffChunk);

					for (FWaveCue& WaveCue : WaveCues)
					{
						if (WaveCue.CuePointID == LabelChunk->CuePointID)
						{
							char* LabelCharText = (char*)(LabelChunk + 1);
							while (*LabelCharText != '\0')
							{
								WaveCue.Label.AppendChar(*LabelCharText);
								++LabelCharText;
							}
							WaveCue.Label.AppendChar('\0');
							break;
						}
					}
				}
				RiffChunk = (FRiffChunkOld*)((uint8*)RiffChunk + Pad16Bit(INTEL_ORDER32(RiffChunk->ChunkLen)) + 8);
			}
		}
	}

	// Look for the cue chunks
	RiffChunk = FindRiffChunk(RiffChunkStart, WaveDataEnd, UE_mmioFOURCC('b', 'e', 'x', 't'));

	if (RiffChunk != nullptr)
	{
		const FRiffBroadcastAudioExtension* BextChunk = (const FRiffBroadcastAudioExtension*)((uint8*)RiffChunk);
		uint64 NumSamplesSinceMidnight = ((uint64) BextChunk->TimeReferenceHigh << 32) | (uint64) BextChunk->TimeReferenceLow;

		// Only record info if there's a valid non-zero time-code.
		if(NumSamplesSinceMidnight > 0 && FmtChunk && FmtChunk->nSamplesPerSec)
		{
			TimecodeInfo = MakePimpl<FSoundWaveTimecodeInfo, EPimplPtrMode::DeepCopy>();

			TimecodeInfo->NumSamplesSinceMidnight   = NumSamplesSinceMidnight;
			TimecodeInfo->NumSamplesPerSecond		= FmtChunk->nSamplesPerSec;
			
			// BextChunk char arrays have a fixed length and might not be null terminated or padded
			// Safely retrieve either the first null termination or the safe max length of the char array
			FFileHelper::BufferToString(TimecodeInfo->Description, BextChunk->Description, sizeof(BextChunk->Description));
			FFileHelper::BufferToString(TimecodeInfo->OriginatorDescription, BextChunk->Originator, sizeof(BextChunk->Originator));
			FFileHelper::BufferToString(TimecodeInfo->OriginatorDate, BextChunk->OriginationDate, sizeof(BextChunk->OriginationDate));
			FFileHelper::BufferToString(TimecodeInfo->OriginatorReference, BextChunk->OriginatorReference, sizeof(BextChunk->OriginatorReference));
			FFileHelper::BufferToString(TimecodeInfo->OriginatorTime, BextChunk->OriginationTime, sizeof(BextChunk->OriginationTime));

			// Use the sample rate as the timecode rate for now. We'll replace
			// it with a more accurate rate if possible below.
			TimecodeInfo->TimecodeRate = FFrameRate(TimecodeInfo->NumSamplesPerSecond, 1u);
		}
	}

	// Look for the cue chunks
	RiffChunk = FindRiffChunk(RiffChunkStart, WaveDataEnd, UE_mmioFOURCC('i', 'X', 'M', 'L'));

	// If we got timecode info above, extend it with the timecode rate and drop
	// frame flag if they are present in the XML.
	if (TimecodeInfo.IsValid() && RiffChunk != nullptr)
	{
		const FRiffIXmlChunk* IXmlChunk = (const FRiffIXmlChunk*)((uint8*)RiffChunk);
		const FString XmlString = StringCast<TCHAR>((ANSICHAR*)IXmlChunk->XmlText).Get();

		// Detail on the iXML specification here: http://www.gallery.co.uk/ixml/
		const FXmlFile RiffXml(XmlString, EConstructMethod::ConstructFromBuffer);
		if (RiffXml.IsValid())
		{
			const FString SpeedTag("SPEED");
			const FString TimecodeRateTag("TIMECODE_RATE");
			const FString TimecodeRateDelimiter("/");
			const FString TimecodeFlagTag("TIMECODE_FLAG");
			const FString DropFrameFlag("DF");

			const FXmlNode* RootNode = RiffXml.GetRootNode();
			const FXmlNode* SpeedNode = RootNode->FindChildNode(SpeedTag);
			const FXmlNode* TimecodeRateNode = SpeedNode ? SpeedNode->FindChildNode(TimecodeRateTag) : nullptr;
			if (TimecodeRateNode)
			{
				const FString TimecodeRateContent = TimecodeRateNode->GetContent();
				TArray<FString> TimecodeRateParts;
				TimecodeRateContent.ParseIntoArray(TimecodeRateParts, *TimecodeRateDelimiter);
				if (TimecodeRateParts.Num() == 2)
				{
					uint32 TimecodeRateNumerator = 0u;
					uint32 TimecodeRateDenominator = 0u;
					LexFromString(TimecodeRateNumerator, *TimecodeRateParts[0]);
					LexFromString(TimecodeRateDenominator, *TimecodeRateParts[1]);
					if (TimecodeRateNumerator != 0u && TimecodeRateDenominator != 0u)
					{
						TimecodeInfo->TimecodeRate = FFrameRate(TimecodeRateNumerator, TimecodeRateDenominator);
					}
				}
			}

			const FXmlNode* TimecodeFlagNode = SpeedNode ? SpeedNode->FindChildNode(TimecodeFlagTag) : nullptr;
			if (TimecodeFlagNode)
			{
				const FString TimecodeFlagContent = TimecodeFlagNode->GetContent();
				if (TimecodeFlagContent == DropFrameFlag)
				{
					TimecodeInfo->bTimecodeIsDropFrame = true;
				}
			}
		}
	}

	// Look for smpl chunk
	RiffChunk = FindRiffChunk(RiffChunkStart, WaveDataEnd, UE_mmioFOURCC('s', 'm', 'p', 'l'));

	if (RiffChunk != nullptr)
	{
		const FRiffSampleChunk* SampleChunk = (const FRiffSampleChunk*)(RiffChunk);

		// only swap the members that we care about
#if !PLATFORM_LITTLE_ENDIAN
		if (!AlreadySwapped)
		{
			SampleChunk->NumSampleLoops = INTEL_ORDER32(SampleChunk->NumSampleLoops);
		}
#endif
		WaveSampleLoops.Reset(SampleChunk->NumSampleLoops);

		// use the RiffSampleChunk size to offset into the chunk data and get the list of SampleLoops
		FRiffSampleLoopChunk* SampleLoopChunks = (FRiffSampleLoopChunk*)((uint8*)RiffChunk + sizeof(FRiffSampleChunk));
		for (uint32 LoopIdx = 0; LoopIdx < SampleChunk->NumSampleLoops; ++LoopIdx)
		{
			FRiffSampleLoopChunk& SampleLoopChunk = SampleLoopChunks[LoopIdx];

#if !PLATFORM_LITTLE_ENDIAN
			if (!AlreadySwapped)
			{
				SampleLoopChunk.LoopID = INTEL_ORDER32(SampleLoopChunk.LoopID);
				SampleLoopChunk.StartFrame = INTEL_ORDER32(SampleLoopChunk.StartFrame);
				SampleLoopChunk.EndFrame = INTEL_ORDER32(SampleLoopChunk.EndFrame);
			}
#endif
			FWaveSampleLoop NewSampleLoop;
			NewSampleLoop.LoopID = SampleLoopChunk.LoopID;
			NewSampleLoop.StartFrame = SampleLoopChunk.StartFrame;
			NewSampleLoop.EndFrame = SampleLoopChunk.EndFrame;
			WaveSampleLoops.Add(NewSampleLoop);
		}
	}

	return true;
}

bool FWaveModInfo::ReadWaveHeader(const uint8* RawWaveData, int32 Size, int32 Offset )
{
	if( Size == 0 )
	{
		return( false );
	}

	// Parse wave info.
	if( !ReadWaveInfo( RawWaveData + Offset, Size ) )
	{
		return( false );
	}

	// Validate the info
	if( ( *pChannels != 1 && *pChannels != 2 ) || *pBitsPerSample != 16 )
	{
		return( false );
	}

	return( true );
}

void FWaveModInfo::ReportImportFailure() const
{
	if (FEngineAnalytics::IsAvailable())
	{
		TArray< FAnalyticsEventAttribute > WaveImportFailureAttributes;
		WaveImportFailureAttributes.Add(FAnalyticsEventAttribute(TEXT("Format"), *pFormatTag));
		WaveImportFailureAttributes.Add(FAnalyticsEventAttribute(TEXT("Channels"), *pChannels));
		WaveImportFailureAttributes.Add(FAnalyticsEventAttribute(TEXT("BitsPerSample"), *pBitsPerSample));

		FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Usage.WaveImportFailure"), WaveImportFailureAttributes);
	}
}

uint32 FWaveModInfo::GetNumSamples() const
{
	// The calculation below only works for uncompressed formats.
	// For compressed formats see Audio::SoundFileUtils::GetNumSamples().
	if (IsFormatUncompressed() && *pBitsPerSample >= 8)
	{
		return SampleDataSize / (*pBitsPerSample / 8);
	}

	return 0;
}

bool FWaveModInfo::IsFormatSupported() const
{
	return (*pFormatTag == WAVE_INFO_FORMAT_PCM
		|| *pFormatTag == WAVE_INFO_FORMAT_ADPCM
		|| *pFormatTag == WAVE_INFO_FORMAT_DVI_ADPCM
		|| *pFormatTag == WAVE_INFO_FORMAT_IEEE_FLOAT
		|| *pFormatTag == WAVE_INFO_FORMAT_OODLE_WAVE);
}

bool FWaveModInfo::IsFormatUncompressed() const
{
	return (*pFormatTag == WAVE_INFO_FORMAT_PCM
		|| *pFormatTag == WAVE_INFO_FORMAT_IEEE_FLOAT);
}

static void WriteUInt32ToByteArrayLE(TArray<uint8>& InByteArray, int32& Index, const uint32 Value)
{
	InByteArray[Index++] = (uint8)(Value >> 0);
	InByteArray[Index++] = (uint8)(Value >> 8);
	InByteArray[Index++] = (uint8)(Value >> 16);
	InByteArray[Index++] = (uint8)(Value >> 24);
}

static void WriteUInt16ToByteArrayLE(TArray<uint8>& InByteArray, int32& Index, const uint16 Value)
{
	InByteArray[Index++] = (uint8)(Value >> 0);
	InByteArray[Index++] = (uint8)(Value >> 8);
}

void SerializeWaveFile(TArray<uint8>& OutWaveFileData, const uint8* InPCMData, const int32 NumBytes, const int32 NumChannels, const int32 SampleRate)
{
	// Reserve space for the raw wave data
	OutWaveFileData.Empty(NumBytes + 44);
	OutWaveFileData.AddZeroed(NumBytes + 44);

	int32 WaveDataByteIndex = 0;

	// Wave Format Serialization ----------

	// FieldName: ChunkID
	// FieldSize: 4 bytes
	// FieldValue: RIFF (FourCC value, big-endian)
	OutWaveFileData[WaveDataByteIndex++] = 'R';
	OutWaveFileData[WaveDataByteIndex++] = 'I';
	OutWaveFileData[WaveDataByteIndex++] = 'F';
	OutWaveFileData[WaveDataByteIndex++] = 'F';

	// ChunkName: ChunkSize: 4 bytes
	// Value: NumBytes + 36. Size of the rest of the chunk following this number. Size of entire file minus 8 bytes.
	WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, NumBytes + 36);

	// FieldName: Format
	// FieldSize: 4 bytes
	// FieldValue: "WAVE"  (big-endian)
	OutWaveFileData[WaveDataByteIndex++] = 'W';
	OutWaveFileData[WaveDataByteIndex++] = 'A';
	OutWaveFileData[WaveDataByteIndex++] = 'V';
	OutWaveFileData[WaveDataByteIndex++] = 'E';

	// FieldName: Subchunk1ID
	// FieldSize: 4 bytes
	// FieldValue: "fmt "
	OutWaveFileData[WaveDataByteIndex++] = 'f';
	OutWaveFileData[WaveDataByteIndex++] = 'm';
	OutWaveFileData[WaveDataByteIndex++] = 't';
	OutWaveFileData[WaveDataByteIndex++] = ' ';

	// FieldName: Subchunk1Size
	// FieldSize: 4 bytes
	// FieldValue: 16 for PCM
	WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, 16);

	// FieldName: AudioFormat
	// FieldSize: 2 bytes
	// FieldValue: 1 for PCM
	WriteUInt16ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, 1);

	// FieldName: NumChannels
	// FieldSize: 2 bytes
	// FieldValue: 1 for for mono
	WriteUInt16ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, NumChannels);

	// FieldName: SampleRate
	// FieldSize: 4 bytes
	// FieldValue: Passed in sample rate
	WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, SampleRate);

	// FieldName: ByteRate
	// FieldSize: 4 bytes
	// FieldValue: SampleRate * NumChannels * BitsPerSample/8
	int32 ByteRate = SampleRate * NumChannels * 2;
	WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, ByteRate);

	// FieldName: BlockAlign
	// FieldSize: 2 bytes
	// FieldValue: NumChannels * BitsPerSample/8
	int32 BlockAlign = NumChannels * 2;
	WriteUInt16ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, BlockAlign);

	// FieldName: BitsPerSample
	// FieldSize: 2 bytes
	// FieldValue: 16 (16 bits per sample)
	WriteUInt16ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, 16);

	// FieldName: Subchunk2ID
	// FieldSize: 4 bytes
	// FieldValue: "data" (big endian)

	OutWaveFileData[WaveDataByteIndex++] = 'd';
	OutWaveFileData[WaveDataByteIndex++] = 'a';
	OutWaveFileData[WaveDataByteIndex++] = 't';
	OutWaveFileData[WaveDataByteIndex++] = 'a';

	// FieldName: Subchunk2Size
	// FieldSize: 4 bytes
	// FieldValue: number of bytes of the data
	WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, NumBytes);

	// Copy the raw PCM data to the audio file
	FMemory::Memcpy(&OutWaveFileData[WaveDataByteIndex], InPCMData, NumBytes);
}


