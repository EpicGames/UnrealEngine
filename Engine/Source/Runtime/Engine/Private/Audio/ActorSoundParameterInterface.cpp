// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/ActorSoundParameterInterface.h"

#include "AudioParameter.h"
#include "HAL/IConsoleManager.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorSoundParameterInterface)

namespace ActorSoundParameterInterfaceConsoleVariables
{
	bool bGatherImplementers = false;
	FAutoConsoleVariableRef CVarGatherImplementers(
		TEXT("au.ActorSoundParameterInterface.GatherImplementers"),
		bGatherImplementers,
		TEXT("When true, allows the interface to search for attached components and actors that implement the interface."),
		ECVF_Default);

} // namespace ActorSoundParameterInterfaceConsoleVariables

void UActorSoundParameterInterface::Fill(const AActor* OwningActor, TArray<FAudioParameter>& OutParams)
{
	QUICK_SCOPE_CYCLE_COUNTER(UActorSoundParameterInterface_Fill);

	if (!ActorSoundParameterInterfaceConsoleVariables::bGatherImplementers)
	{
		// Prior to the GetImplementers change, we only considered the owning actor, and no attached actors or components.
		if (OwningActor && OwningActor->Implements<UActorSoundParameterInterface>())
		{
			IActorSoundParameterInterface::Execute_GetActorSoundParams(OwningActor, OutParams);
		}
		return;
	}
	else
	{
		TArray<const AActor*> Actors;
		TArray<const UActorComponent*> Components;
		// This is prohibitively expensive, as it goes through our owning actor and all attached actors, looking for 
		// components that implement the IActorSoundParameterInterface
		GetImplementers(OwningActor, Actors, Components);

		for (const AActor* Actor : Actors)
		{
			IActorSoundParameterInterface::Execute_GetActorSoundParams(Actor, OutParams);
		}

		for (const UActorComponent* Component : Components)
		{
			IActorSoundParameterInterface::Execute_GetActorSoundParams(Component, OutParams);
		}
	}
}

void UActorSoundParameterInterface::GetImplementers(const AActor* InActor, TArray<const AActor*>& OutActors, TArray<const UActorComponent*>& OutComponents)
{
	QUICK_SCOPE_CYCLE_COUNTER(UActorSoundParameterInterface_GetImplementers);

	if (!InActor)
	{
		return;
	}

	// Helper to collect objects that implement this interface from an actor (and its components)
	auto CollectFromActor = [&OutActors, &OutComponents](const AActor* InActor)
	{
		if (InActor)
		{
			if (InActor->Implements<UActorSoundParameterInterface>())
			{
				OutActors.Add(InActor);
			}

			TArray<UActorComponent*> Components = InActor->GetComponentsByInterface(UActorSoundParameterInterface::StaticClass());
			OutComponents.Append(Components);
		}
	};

	// Collect Actors/Components that implement this interface
	const AActor* RootActor = InActor;
	if (USceneComponent* RootComp = RootActor->GetRootComponent())
	{
		// Walk up to the top-most attach actor in the hierarchy (will just be the RootActor if no attachment)
		RootActor = RootComp->GetAttachmentRootActor();
	}

	CollectFromActor(RootActor);

	// Grab all attached actors (recursive)
	TArray<AActor*> AttachedActors;
	constexpr bool bResetArray = false;
	constexpr bool bRecursivelyIncludeAttachedActors = true;
	RootActor->GetAttachedActors(AttachedActors, bResetArray, bRecursivelyIncludeAttachedActors);

	for (const AActor* Actor : AttachedActors)
	{
		CollectFromActor(Actor);
	}
}

