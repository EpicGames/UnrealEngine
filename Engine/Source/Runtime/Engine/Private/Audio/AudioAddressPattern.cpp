// Copyright Epic Games, Inc. All Rights Reserved.
// Pattern-Matching based on OSC 1.0 Protocol
#include "Audio/AudioAddressPattern.h"
#include "CoreGlobals.h"


bool FAudioAddressPattern::BracePatternMatches(const FString& Pattern, int32 PatternStartIndex, int32 PatternEndIndex, const FString& Part, int32& PartIter)
{
	// If these checks are hit, function is being called prior to pattern
	// validation or pattern validation is failing to validate properly.
	check(PatternEndIndex - PatternStartIndex > 1);
	check(Pattern[PatternStartIndex] == '{');
	check(Pattern[PatternEndIndex] == '}');

	// 1. Empty pattern, so always true
	if (Pattern.Len() == 2)
	{
		return true;
	}

	// Increment start index to match first slot in strings to match
	int32 PatternIter = PatternStartIndex + 1;
	bool bMatches = false;
	for (int32 PartIndex = PartIter; PartIndex < Part.Len() && PatternIter < PatternEndIndex;)
	{
		// End of matching string.
		if (Pattern[PatternIter] == ',')
		{
			// Check to see if succeeded and return state if so.
			if (bMatches)
			{
				PartIter = PartIndex;
				return true;
			}
			// If not, reset part search and 
			// continue to next Pattern string.
			PartIndex = PartIter;
		}
		// Mark as matching if we're on the path to a match.
		else if (Pattern[PatternIter] == Part[PartIndex])
		{
			bMatches = true;
			++PartIndex;
		}
		// Reset part iterator and fast forward to next pattern string to match
		else
		{
			bMatches = false;
			PartIndex = PartIter;
			while (PatternIter < PatternEndIndex && Pattern[PatternIter] != ',')
			{
				++PatternIter;
			}
		}
		++PatternIter;
	}

	return false;
}

bool FAudioAddressPattern::BracketPatternMatches(const FString& Pattern, int32 PatternStartIndex, int32 PatternEndIndex, TCHAR MatchChar)
{
	// If these checks are hit, function is being called prior to pattern
	// validation or pattern validation is failing to validate properly.
	check(PatternEndIndex - PatternStartIndex > 1);
	check(Pattern[PatternStartIndex] == '[');
	check(Pattern[PatternEndIndex] == ']');

	// 1. Empty pattern, so always true
	if (Pattern.Len() == 2)
	{
		return true;
	}

	// Prime start index and cache whether or not negation of pattern is requested.
	PatternStartIndex++;
	bool bNegate = false;
	if (Pattern[1] == '!')
	{
		// Of non-standard form [!]: OSC 1.0 Standard is non-descript about this
		// scenario, so treat as if this case negates any additional character.
		if (Pattern.Len() == 3)
		{
			return false;
		}

		PatternStartIndex++;
		bNegate = true;
	}

	// 2. Special case where in the form [a-z] or [!a-z]
	if (Pattern.Len() == PatternStartIndex + 4)
	{
		if (Pattern[PatternStartIndex + 1] == '-')
		{
			if (MatchChar >= Pattern[PatternStartIndex] && MatchChar <= Pattern[PatternStartIndex + 2])
			{
				return !bNegate;
			}
			else if (MatchChar >= Pattern[PatternStartIndex + 2] && MatchChar <= Pattern[PatternStartIndex])
			{
				return !bNegate;
			}

			return bNegate;
		}
	}

	// 4. Form is either [!abcdef] or [abcdef]
	for (int32 i = PatternStartIndex; i < PatternEndIndex; ++i)
	{
		if (MatchChar == Pattern[i])
		{
			return !bNegate;
		}
	}

	return bNegate;
}

int32 FAudioAddressPattern::FindPatternTerminatorIndex(const FString& Pattern, int32 PatternIter, TCHAR Terminator)
{
	int32 EndIndex = PatternIter + 1;
	for (;Pattern[EndIndex] != Terminator; ++EndIndex)
	{
		// Should never hit as this function should never be called on invalid pattern
		check(EndIndex < Pattern.Len());
	}
	return EndIndex;
}

const TArray<TCHAR>& FAudioAddressPattern::GetInvalidChars()
{
	static const TArray<TCHAR> InvalidChars = { ' ', '#', };
	return InvalidChars;
}

const TArray<TCHAR>& FAudioAddressPattern::GetPatternChars()
{
	static const TArray<TCHAR> PatternChars = { ',', '*', '?', '[', ']', '{', '}' };
	return PatternChars;
}

bool FAudioAddressPattern::IsValidPatternPart(const FString& Part)
{
	bool bInBrackets = false;
	bool bInBraces = false;

	for (TCHAR Char : Part.GetCharArray())
	{
		if (Char == '/')
		{
			return false;
		}

		if (GetInvalidChars().Contains(Char))
		{
			return false;
		}

		switch (Char)
		{
		case ',':
		{
			if (!bInBraces)
			{
				return false;
			}
		}
		break;

		case '?':
		case '*':
		{
			if (bInBraces || bInBrackets)
			{
				return false;
			}
		}
		break;

		case '[':
		{
			// Nested bracket/brace case
			if (bInBraces || bInBrackets)
			{
				return false;
			}
			bInBrackets = true;
		}
		break;

		case ']':
		{
			// Mixed bracket/brace case
			if (bInBraces)
			{
				return false;
			}
			bInBrackets = false;
		}
		break;

		case '{':
		{
			// Nested bracket/brace case
			if (bInBraces || bInBrackets)
			{
				return false;
			}
			bInBraces = true;
		}
		break;

		case '}':
		{
			// Mixed bracket/brace case
			if (bInBrackets)
			{
				return false;
			}
			bInBraces = false;
		}
		break;
		}
	}

	// Missing close braces/brackets
	return !bInBraces && !bInBrackets;
}

bool FAudioAddressPattern::IsValidPattern(const TArray<FString>& InContainers, const FString& InMethod)
{
	if (InContainers.Num() == 0 || InMethod.IsEmpty())
	{
		return false;

	}

	for (const FString& Container : InContainers)
	{
		if (!IsValidPatternPart(Container))
		{
			return false;
		}
	}

	return IsValidPatternPart(InMethod);
}

bool FAudioAddressPattern::IsValidPath(const FString& Path, bool bInvalidateSeparator)
{
	if (Path.IsEmpty() || Path == TEXT("/"))
	{
		return false;
	}

	for (TCHAR Char : Path.GetCharArray())
	{
		if (Char == '/')
		{
			if (bInvalidateSeparator)
			{
				return false;
			}
			continue;
		}

		if (GetInvalidChars().Contains(Char))
		{
			return false;
		}

		if (GetPatternChars().Contains(Char))
		{
			return false;
		}
	}

	return true;
}

bool FAudioAddressPattern::PartsMatch(const FString& Pattern, const FString& Part)
{
	int32 PartIter = 0;
	for (int32 PatternIter = 0; PatternIter < Pattern.Len() && PartIter < Part.Len();)
	{
		// If pattern is at end but part still has characters to process, match fails
		if (PatternIter >= Pattern.Len() && PartIter < Part.Len())
		{
			return false;
		}

		// If part is at end but pattern still has rules to process, match fails
		if (PatternIter < Pattern.Len() && PartIter >= Part.Len())
		{
			return false;
		}

		switch (Pattern[PatternIter])
		{
		case '?':
		{
			// ignore case as any ol' input path character will do
			PatternIter++;
			PartIter++;
		}
		break;

		case '*':
		{
			if (!WildPatternMatches(Pattern, PatternIter, Part, PartIter))
			{
				return false;
			}
			PatternIter++;
			PartIter++;
		}
		break;

		case '[':
		{
			int32 PatternEndIndex = FindPatternTerminatorIndex(Pattern, PatternIter, ']');
			if (!BracketPatternMatches(Pattern, PatternIter, PatternEndIndex, Part[PartIter]))
			{
				return false;
			}
			PatternIter = PatternEndIndex + 1;
			PartIter++;
		}
		break;

		case '{':
		{
			int32 PatternEndIndex = FindPatternTerminatorIndex(Pattern, PatternIter, '}');
			if (!BracePatternMatches(Pattern, PatternIter, PatternEndIndex, Part, PartIter))
			{
				return false;
			}
			PatternIter = PatternEndIndex + 1;
			PartIter++;
		}
		break;

		// All other valid characters direct test
		default:
		{
			if (Pattern[PatternIter] != Part[PartIter])
			{
				return false;
			}
			PatternIter++;
			PartIter++;
		}
		break;
		}
	}
	return true;
}

bool FAudioAddressPattern::WildPatternMatches(const FString& Pattern, int32& PatternIter, const FString& Part, int32& PartIter)
{
	// Filter out/bump pattern index for case of having sequential wildcards
	for (; PatternIter + 1 < Pattern.Len(); ++PatternIter)
	{
		if (Pattern[PatternIter + 1] != '*')
		{
			break;
		}
	}

	// Increment to start of next thing to match next test.  Have to always run following test
	// here as if it fails, the part iter can be incremented, effectively allowing the lead character to be
	// "consumed" by the wildcard and the following test has to be run again until search is exhausted
	// or following test is fulfilled.
	++PatternIter;

	// Reached end of pattern, which is a '*' so no more parsing on this part char required 
	// (all path part beyond is considered valid)
	if (PatternIter == Pattern.Len())
	{
		return true;
	}

	const bool bWildLeadsBrackets = Pattern[PatternIter] == '[';
	const bool bWildLeadsBraces = Pattern[PatternIter] == '{';
	const TCHAR Terminator = bWildLeadsBrackets ? ']' : '}';

	int32 PatternEndIndex = 0;
	if (bWildLeadsBrackets || bWildLeadsBraces)
	{
		PatternEndIndex = FindPatternTerminatorIndex(Pattern, PatternIter, Terminator);
	}

	// Continue bumping PartIter until either at end of Part or Part char matches next pattern
	for (;PartIter < Part.Len(); ++PartIter)
	{
		if (bWildLeadsBrackets)
		{
			if (BracketPatternMatches(Pattern, PatternIter, PatternEndIndex, Part[PartIter]))
			{
				PatternIter = PatternEndIndex;
				return true;
			}
		}
		else if (bWildLeadsBraces)
		{
			// Brace match updates OutPartIter.  Only pass along and update end of brace section
			// if it succeeds to avoid corrupting initial state for subsequent tests after failing.
			int32 OutPartIter = PartIter;
			if (BracePatternMatches(Pattern, PatternIter, PatternEndIndex, Part, OutPartIter))
			{
				PatternIter = PatternEndIndex;
				PartIter = OutPartIter;
				return true;
			}
		}
		else
		{
			if (Pattern[PatternIter] == Part[PartIter] || Pattern[PatternIter] == '?')
			{
				return true;
			}
		}
	}

	return false;
}
