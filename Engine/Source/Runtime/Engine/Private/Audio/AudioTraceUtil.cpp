// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/AudioTraceUtil.h"

#include "ActiveSound.h"
#include "AudioDevice.h"
#include "Styling/SlateIconFinder.h"
#include "Trace/Trace.h"

#if UE_AUDIO_PROFILERTRACE_ENABLED

UE_TRACE_EVENT_BEGIN(Audio, EventLog)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AssetPath)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, EventLogName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorLabel)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorIconName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SoundClassName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AudioComponentName)
UE_TRACE_EVENT_END()

namespace Audio::Trace
{
	namespace Private
	{
		const FString MetaSoundSource = "MetaSoundSource";
		const FString SoundWaveProcedural = "SoundWaveProcedural";
		const FString SoundCue = "SoundCue";
		const FString SoundWave = "SoundWave";
		const FString SoundCueTemplate = "SoundCueTemplate";

		bool NameMatchesBaseAudioAsset(const FString& SoundClassName)
		{
			return SoundClassName == MetaSoundSource
				|| SoundClassName == SoundWaveProcedural
				|| SoundClassName == SoundCue
				|| SoundClassName == SoundWave
				|| SoundClassName == SoundCueTemplate;
		}
	}

	namespace Util
	{
		FString GetAudioComponentName(const FActiveSound& InActiveSound)
		{
			if (const TObjectPtr<UAudioComponent> AudioComponent = UAudioComponent::GetAudioComponentFromID(InActiveSound.GetAudioComponentID()))
			{
				return AudioComponent->GetName();
			}

			return FString();
		}

		FString GetSoundBaseAssetName(const TObjectPtr<UClass> SoundClass)
		{
			if (SoundClass == nullptr)
			{
				return FString();
			}

			TObjectPtr<UClass> SoundSuperClass = SoundClass->GetSuperClass();
			FString SoundClassName = SoundClass->GetName();
			while (!Private::NameMatchesBaseAudioAsset(SoundClassName) && !SoundClassName.IsEmpty())
			{
				SoundClassName = SoundSuperClass ? SoundSuperClass->GetName() : FString();
				SoundSuperClass = SoundSuperClass ? SoundSuperClass->GetSuperClass() : nullptr;
			}

			return SoundClassName;
		}

		FString GetOwnerActorLabel(const FActiveSound& InActiveSound)
		{
			if (const TObjectPtr<UAudioComponent> AudioComponent = UAudioComponent::GetAudioComponentFromID(InActiveSound.GetAudioComponentID()))
			{
				if (const AActor* OwnerActor = AudioComponent->GetOwner())
				{
					return OwnerActor->GetActorNameOrLabel();
				}
			}

			return InActiveSound.GetOwnerName();
		}

		FName GetOwnerActorIconName(const FActiveSound& InActiveSound)
		{
#if WITH_EDITOR
			if (const TObjectPtr<UAudioComponent> AudioComponent = UAudioComponent::GetAudioComponentFromID(InActiveSound.GetAudioComponentID()))
			{
				if (const AActor* OwnerActor = AudioComponent->GetOwner())
				{
					FName IconName = OwnerActor->GetCustomIconName();
					if (IconName == NAME_None)
					{
						// Actor didn't specify an icon - fallback on the class icon
						return FSlateIconFinder::FindIconForClass(OwnerActor->GetClass()).GetStyleName();
					}
				}
			}
#endif // WITH_EDITOR

			return NAME_None;
		}

		TObjectPtr<UObject> GetSoundObjectPointer(const FActiveSound& InActiveSound)
		{
			return InActiveSound.GetSound() != nullptr ? FSoftObjectPath(InActiveSound.GetSound()->GetPathName()).ResolveObject() : nullptr;
		}
	} // namespace Util

	namespace EventLog
	{
		void SendActiveSoundEvent(const FActiveSound& InActiveSound, const FString& Event)
		{
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);

			if (bChannelEnabled && InActiveSound.AudioDevice)
			{
				const TObjectPtr<UObject> SoundObjectPtr = Util::GetSoundObjectPointer(InActiveSound);
				if (SoundObjectPtr == nullptr)
				{
					return;
				}

				SendEvent(InActiveSound.AudioDevice->DeviceID
						, Event
						, InActiveSound.GetPlayOrder()
						, SoundObjectPtr->GetPathName()
						, Util::GetOwnerActorLabel(InActiveSound)
						, Util::GetOwnerActorIconName(InActiveSound).ToString()
						, Util::GetSoundBaseAssetName(SoundObjectPtr->GetClass())
						, Util::GetAudioComponentName(InActiveSound));
			}
		}

		void SendEvent(const Audio::FDeviceId AudioDeviceID, const FString& Event, const uint32 AudioObjectID, const FString& AssetPath, const FString& ActorLabel, const FString& ActorIconName, const FString& SoundClassName, const FString& AudioComponentName)
		{
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);

			if (bChannelEnabled)
			{
				UE_TRACE_LOG(Audio, EventLog, AudioChannel)
					<< EventLog.DeviceId(AudioDeviceID)
					<< EventLog.Timestamp(FPlatformTime::Cycles64())
					<< EventLog.PlayOrder(AudioObjectID)
					<< EventLog.AssetPath(*AssetPath)
					<< EventLog.EventLogName(*Event)
					<< EventLog.ActorLabel(*ActorLabel)
					<< EventLog.ActorIconName(*ActorIconName)
					<< EventLog.SoundClassName(*SoundClassName)
					<< EventLog.AudioComponentName(*AudioComponentName);
			}
		}
	} // namespace EventLog

} // namespace Audio::AudioTraceUtil

#endif // UE_AUDIO_PROFILERTRACE_ENABLED