// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/AudioWidgetSubsystem.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "Blueprint/UserWidget.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "UObject/CoreRedirects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioWidgetSubsystem)


#define LOCTEXT_NAMESPACE "AudioWidgetSubsystem"


TArray<UUserWidget*> UAudioWidgetSubsystem::CreateUserWidgets(
	UWorld& InWorld,
	TSubclassOf<UInterface> InWidgetInterface,
	TFunction<bool(UUserWidget*)> FilterFunction,
	TFunction<bool(UBlueprint*)> BlueprintFilterFunction) const
{
	TArray<UUserWidget*> UserWidgets;
	UClass* InterfaceClass = InWidgetInterface ? InWidgetInterface.Get() : UInterface::StaticClass();
	if (!ensure(InterfaceClass))
	{
		return UserWidgets;
	}

	const TArray<FAssetData> AssetData = GetBlueprintAssetData(InterfaceClass);
	for (const FAssetData& AssetEntry : AssetData)
	{
		UObject* Asset = AssetEntry.GetAsset();
		if (!Asset)
		{
			continue;
		}

		UBlueprint* Blueprint = CastChecked<UBlueprint>(Asset);
		UClass* GeneratedClass = Blueprint->GeneratedClass;
		if (!GeneratedClass)
		{
			continue;
		}

		if (BlueprintFilterFunction.IsSet() && !BlueprintFilterFunction(Blueprint))
		{
			continue;
		}

		if (UUserWidget * UserWidget = CreateWidget<UUserWidget>(&InWorld, GeneratedClass))
		{
			if (!FilterFunction || FilterFunction(UserWidget))
			{
				UserWidgets.Add(UserWidget);
			}
		}
	}

	return UserWidgets;
}

const TArray<FAssetData> UAudioWidgetSubsystem::GetBlueprintAssetData(UClass* InInterfaceClass)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> AssetData;

	FARFilter Filter;
	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	Filter.ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/UMGEditor"), TEXT("WidgetBlueprint")));
	Filter.ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/Blutility"), TEXT("EditorUtilityWidget")));
	Filter.bRecursiveClasses = true;

	AssetRegistryModule.Get().EnumerateAssets(Filter, [FoundAssets = &AssetData, InInterfaceClass](const FAssetData& InAssetData)
	{
		if (ImplementsInterface(InAssetData, InInterfaceClass))
		{
			FoundAssets->Add(InAssetData);
		}

		return true;
	});

	return AssetData;
}

bool UAudioWidgetSubsystem::ImplementsInterface(const FAssetData& InAssetData, UClass* InInterfaceClass)
{
	UClass* NativeClass = UAssetRegistryHelpers::FindAssetNativeClass(InAssetData);
	// Native interfaces can't use the BP tag system
	if (InAssetData.GetClass()->ImplementsInterface(InInterfaceClass))
	{
		return true;
	}

	const FString ImplementedInterfaces = InAssetData.GetTagValueRef<FString>(FBlueprintTags::ImplementedInterfaces);
	if (!ImplementedInterfaces.IsEmpty())
	{
		FString CurrentString = *ImplementedInterfaces;

		static const FString InterfaceIdentifier = "Interface=";
		int32 FirstIndex = CurrentString.Find(InterfaceIdentifier);
		if (FirstIndex > 0)
		{
			CurrentString.RightChopInline(FirstIndex);
		}

		TArray<FString> Tokens;
		CurrentString.ParseIntoArray(Tokens, *InterfaceIdentifier);
		for (FString& Token : Tokens)
		{
			FirstIndex = Token.Find(TEXT("'"));
			if (FirstIndex > 0)
			{
				Token.RightChopInline(FirstIndex + 1);
			}

			Token.RemoveFromStart(TEXT("\""));

			FirstIndex = Token.Find(TEXT("'"));
			if (FirstIndex > 0)
			{
				Token.LeftChopInline(Token.Len() - FirstIndex);
			}

			Token.RemoveFromEnd(TEXT("\""));

			const FCoreRedirectObjectName ResolvedInterfaceName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(Token));
			if (InInterfaceClass->GetFName() == ResolvedInterfaceName.ObjectName)
			{
				return true;
			}
		}
	}

	return false;
}
#undef LOCTEXT_NAMESPACE // AudioWidgetSubsystem
