// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/SoundParameterControllerInterface.h"

#include "AudioDevice.h"
#include "IAudioParameterTransmitter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundParameterControllerInterface)


namespace SoundParameterControllerInterfacePrivate
{
	static const FName ProxyFeatureName("SoundParameterControllerInterface");
} // namespace SoundParameterControllerInterfacePrivate

USoundParameterControllerInterface::USoundParameterControllerInterface(FObjectInitializer const& InObjectInitializer)
	: Super(InObjectInitializer)
{
}

void ISoundParameterControllerInterface::ResetParameters()
{
	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		if (IsPlaying() && !GetDisableParameterUpdatesWhilePlaying())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SoundParameterControllerInterface.ResetParameters"), STAT_AudioResetParameters, STATGROUP_AudioThreadCommands);
			AudioDevice->SendCommandToActiveSounds(GetInstanceOwnerID(), [] (FActiveSound& ActiveSound)
			{
				if (Audio::IParameterTransmitter* Transmitter = ActiveSound.GetTransmitter())
				{
					Transmitter->ResetParameters();
				}
			}, GET_STATID(STAT_AudioResetParameters));
		}
	}
}

void ISoundParameterControllerInterface::SetTriggerParameter(FName InName)
{
	if (InName.IsNone())
	{
		return;
	}

	if (IsPlaying() && !GetDisableParameterUpdatesWhilePlaying())
	{
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			FAudioParameter ParamToSet = FAudioParameter(InName, EAudioParameterType::Trigger);
			
			if (USoundBase* Sound = GetSound())
			{
				TArray<FAudioParameter> Params = { MoveTemp(ParamToSet) };
				Sound->InitParameters(Params, SoundParameterControllerInterfacePrivate::ProxyFeatureName);
				if (Params.Num() == 0)
				{
					// USoundBase::InitParameters(...) can remove parameters. 
					// Exit early if the parameter is removed.
					return;
				}
				ParamToSet = MoveTemp(Params[0]);
			}

			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SoundParameterControllerInterface.ExecuteTriggerParameter"), STAT_AudioExecuteTriggerParameter, STATGROUP_AudioThreadCommands);

			AudioDevice->SendCommandToActiveSounds(GetInstanceOwnerID(), [AudioDevice, Param = MoveTemp(ParamToSet)](FActiveSound& ActiveSound)
			{
				if (Audio::IParameterTransmitter* Transmitter = ActiveSound.GetTransmitter())
				{
					const FName ParamName = Param.ParamName;

					// Must be copied as original version must be preserved in case command is called on multiple ActiveSounds.
					TArray<FAudioParameter> TempParam = { Param };
					if (!Transmitter->SetParameters(MoveTemp(TempParam)))
					{
						UE_LOGF(LogAudio, Warning, "Failed to execute trigger parameter '%ls'", *ParamName.ToString());
					}
				}
			}, GET_STATID(STAT_AudioExecuteTriggerParameter));
		}
	}
}

void ISoundParameterControllerInterface::SetBoolParameter(FName InName, bool InValue)
{
	SetParameters({ FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetBoolArrayParameter(FName InName, const TArray<bool>& InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetIntParameter(FName InName, int32 InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetIntArrayParameter(FName InName, const TArray<int32>& InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetFloatParameter(FName InName, float InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetFloatArrayParameter(FName InName, const TArray<float>& InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetStringParameter(FName InName, const FString& InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetStringArrayParameter(FName InName, const TArray<FString>& InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetObjectParameter(FName InName, UObject* InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetObjectArrayParameter(FName InName, const TArray<UObject*>& InValue)
{
	SetParameters( { FAudioParameter(InName, InValue) });
}

void ISoundParameterControllerInterface::SetParameter(FAudioParameter&& InValue)
{
	SetParameters({ MoveTemp(InValue) });
}

void ISoundParameterControllerInterface::SetParameters(TArray<FAudioParameter>&& InValues)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ISoundParameterControllerInterface::SetParameters);

	const bool bUpdateActiveSound = IsPlaying() && !GetDisableParameterUpdatesWhilePlaying();

	TArray<FAudioParameter> ParamsToSet;
	if (bUpdateActiveSound)
	{
		ParamsToSet = InValues;
	}

	TArray<FAudioParameter>& InstanceParameters = GetInstanceParameters();
	FAudioParameter::Merge(MoveTemp(InValues), InstanceParameters);

	if (bUpdateActiveSound)
	{
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			if (USoundBase* Sound = GetSound())
			{
				Sound->InitParameters(ParamsToSet, SoundParameterControllerInterfacePrivate::ProxyFeatureName);
			}

			// Prior call to InitParameters can prune parameters if they are
			// invalid, so check here to avoid unnecessary pass of empty array.
			if (!ParamsToSet.IsEmpty())
			{
				DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SoundParameterControllerInterface.SetParameters"), STAT_AudioSetParameters, STATGROUP_AudioThreadCommands);
				AudioDevice->SendCommandToActiveSounds(GetInstanceOwnerID(), [AudioDevice, Params = MoveTemp(ParamsToSet)](FActiveSound& ActiveSound) mutable
				{
					if (Audio::IParameterTransmitter* Transmitter = ActiveSound.GetTransmitter())
					{
						TArray<FAudioParameter> Temp = Params;
						Transmitter->SetParameters(MoveTemp(Temp));
					}
				}, GET_STATID(STAT_AudioSetParameters));
			}
		}
	}
}

void ISoundParameterControllerInterface::SetParameters_Blueprint(const TArray<FAudioParameter>& InValues)
{
	TArray<FAudioParameter> Values = InValues;
	SetParameters(MoveTemp(Values));
}

FAudioParameter UAudioParameterConversionStatics::BooleanToAudioParameter(FName Name, bool Bool) { return { Name, Bool }; }

FAudioParameter UAudioParameterConversionStatics::FloatToAudioParameter(FName Name, float Float) { return { Name, Float }; }

FAudioParameter UAudioParameterConversionStatics::IntegerToAudioParameter(FName Name, int32 Integer) { return { Name, Integer }; }

FAudioParameter UAudioParameterConversionStatics::StringToAudioParameter(FName Name, FString String) { return { Name, MoveTemp(String) }; }

FAudioParameter UAudioParameterConversionStatics::ObjectToAudioParameter(FName Name, UObject* Object) { return { Name, Object }; }

FAudioParameter UAudioParameterConversionStatics::BooleanArrayToAudioParameter(FName Name, TArray<bool> Bools) { return { Name, MoveTemp(Bools) }; }

FAudioParameter UAudioParameterConversionStatics::FloatArrayToAudioParameter(FName Name, TArray<float> Floats) { return { Name, MoveTemp(Floats) }; }

FAudioParameter UAudioParameterConversionStatics::IntegerArrayToAudioParameter(FName Name, TArray<int32> Integers) { return { Name, MoveTemp(Integers) }; }

FAudioParameter UAudioParameterConversionStatics::StringArrayToAudioParameter(FName Name, TArray<FString> Strings) { return { Name, MoveTemp(Strings) }; }

FAudioParameter UAudioParameterConversionStatics::ObjectArrayToAudioParameter(FName Name, TArray<UObject*> Objects) { return { Name, MoveTemp(Objects) }; }
