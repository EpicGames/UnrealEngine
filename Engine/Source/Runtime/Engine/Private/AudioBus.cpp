// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/AudioBus.h"
#include "AudioBusSubsystem.h"
#include "AudioMixerDevice.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioBus)

FAudioBusProxy::FAudioBusProxy(UAudioBus* InAudioBus)
{
	if (InAudioBus)
	{
		AudioBusId = InAudioBus->GetUniqueID();
		NumChannels = InAudioBus->GetNumChannels();
		TypeHash = ::GetTypeHash(InAudioBus);
	}
}


UAudioBus::UAudioBus(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAudioBus::BeginDestroy()
{
	Super::BeginDestroy();

	if (!GEngine)
	{
		return;
	}

	// Make sure we stop all audio bus instances on all devices if this object is getting destroyed
	uint32 AudioBusId = GetUniqueID();

	FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
	if (AudioDeviceManager)
	{
		TArray<FAudioDevice*> AudioDevices = AudioDeviceManager->GetAudioDevices();
		for (FAudioDevice* AudioDevice : AudioDevices)
		{
			UAudioBusSubsystem* AudioBusSubsystem = AudioDevice->GetSubsystem<UAudioBusSubsystem>();
			check(AudioBusSubsystem);
			AudioBusSubsystem->StopAudioBus(Audio::FAudioBusKey(AudioBusId));
		}
	}
}

#if WITH_EDITOR
void UAudioBus::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!PropertyChangedEvent.Property)
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.Property->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAudioBus, AudioBusChannels))
	{
		if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			DeviceManager->IterateOverAllDevices([BusId = GetUniqueID(), Name = GetPathName(), NumChannels = AudioBusChannels](Audio::FDeviceId, FAudioDevice* InDevice)
			{
				UAudioBusSubsystem* AudioBusSubsystem = InDevice->GetSubsystem<UAudioBusSubsystem>();
				check(AudioBusSubsystem);
				Audio::FAudioBusKey AudioBusKey = Audio::FAudioBusKey(BusId);
				AudioBusSubsystem->StopAudioBus(AudioBusKey);
				AudioBusSubsystem->StartAudioBus(AudioBusKey, Name, (int32)NumChannels + 1, false /* bInIsAutomatic */);
			});
		}
	}
}
#endif // WITH_EDITOR

TSharedPtr<Audio::IProxyData> UAudioBus::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	return MakeShared<FAudioBusProxy>(this);
}

