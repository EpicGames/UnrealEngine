// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCompressionSettingsUtils.h"
#include "AudioStreamingCache.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"

#ifndef ENABLE_PLATFORM_COMPRESSION_OVERRIDES
	#define ENABLE_PLATFORM_COMPRESSION_OVERRIDES 0
#endif

#if PLATFORM_ANDROID && ENABLE_PLATFORM_COMPRESSION_OVERRIDES
#include "AndroidRuntimeSettings.h"
#endif

#if PLATFORM_IOS && ENABLE_PLATFORM_COMPRESSION_OVERRIDES
#include "IOSRuntimeSettings.h"
#endif

#include "Misc/ConfigCacheIni.h"

static float CookOverrideCachingIntervalCvar = 60.0f;
FAutoConsoleVariableRef CVarCookOverrideCachingIntervalCVar(
	TEXT("au.editor.CookOverrideCachingInterval"),
	CookOverrideCachingIntervalCvar,
	TEXT("This sets the max latency between when a cook override is changed in the project settings and when it is applied to new audio sources.\n")
	TEXT("n: Time between caching intervals, in seconds."),
	ECVF_Default);


/**
 * This value is the minimum potential usage of the stream cache we feasibly want to support.
 * Setting this to 0.25, for example, cause us to potentially be using 25% of our cache size when we start evicting chunks, worst cast scenario.
 * The trade off is that when this is increased, we add more elements to our cache, thus linearly increasing the CPU complexity of finding a chunk.
 * A minimum cache usage of 1.0f is impossible, because it would require an infinite amount of chunks.
 */
static float MinimumCacheUsageCvar = 0.9f;
FAutoConsoleVariableRef CVarMinimumCacheUsage(
	TEXT("au.streamcaching.MinimumCacheUsage"),
	MinimumCacheUsageCvar,
	TEXT("This value is the minimum potential usage of the stream cache we feasibly want to support. Setting this to 0.25, for example, cause us to potentially be using 25% of our cache size when we start evicting chunks, worst cast scenario.\n")
	TEXT("0.0: limit the number of chunks to our (Cache Size / Max Chunk Size) [0.01-0.99]: Increase our number of chunks to limit disk IO when we have lots of small sounds playing."),
	ECVF_Default);

static float ChunkSlotNumScalarCvar = 1.0f;
FAutoConsoleVariableRef CVarChunkSlotNumScalar(
	TEXT("au.streamcaching.ChunkSlotNumScalar"),
	ChunkSlotNumScalarCvar,
	TEXT("This allows scaling the number of chunk slots pre-allocated.\n")
	TEXT("1.0: is the lower limit"),
	ECVF_Default);

#if ENABLE_PLATFORM_COMPRESSION_OVERRIDES && PLATFORM_ANDROID
const FPlatformRuntimeAudioCompressionOverrides* FPlatformCompressionUtilities::GetRuntimeCompressionOverridesForCurrentPlatform()
{
	static const UAndroidRuntimeSettings* Settings = GetDefault<UAndroidRuntimeSettings>();
	if (Settings)
	{
		return &(Settings->CompressionOverrides);
	}
	return nullptr;
}
#elif ENABLE_PLATFORM_COMPRESSION_OVERRIDES && PLATFORM_IOS
const FPlatformRuntimeAudioCompressionOverrides* FPlatformCompressionUtilities::GetRuntimeCompressionOverridesForCurrentPlatform()
{
	static const UIOSRuntimeSettings* Settings = GetDefault<UIOSRuntimeSettings>();
	if (Settings)
	{
		return &(Settings->CompressionOverrides);
	}
	return nullptr;
}
#elif !ENABLE_PLATFORM_COMPRESSION_OVERRIDES
const FPlatformRuntimeAudioCompressionOverrides* FPlatformCompressionUtilities::GetRuntimeCompressionOverridesForCurrentPlatform()
{
	return nullptr;
}
#endif // ENABLE_PLATFORM_COMPRESSION_OVERRIDES

void CacheAudioCookOverrides(FPlatformAudioCookOverrides& OutOverrides, const TCHAR* InPlatformName=nullptr)
{
	SCOPED_NAMED_EVENT(CacheAudioCookOverrides, FColor::Blue);

	// if the platform was passed in, use it, otherwise, get the runtime platform's name for looking up DDPI
	FString PlatformName = InPlatformName ? FString(InPlatformName) : FString(FPlatformProperties::IniPlatformName());
	
	// now use that platform name to get the ini section out of DDPI
	const FDataDrivenPlatformInfo& PlatformInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName);
	const FString& CategoryName = PlatformInfo.TargetSettingsIniSectionName;

	// if we don't support platform overrides, then return 
	if (CategoryName.Len() == 0)
	{
		OutOverrides = FPlatformAudioCookOverrides();
		return;
	}

	FConfigFile LocalPlatformEngineIni;
	const FConfigFile* PlatformFile = FConfigCacheIni::FindOrLoadPlatformConfig(LocalPlatformEngineIni, TEXT("Engine"), *PlatformName);

	int32 SoundCueQualityIndex = INDEX_NONE;
	if (PlatformFile->GetInt(*CategoryName, TEXT("SoundCueCookQualityIndex"), SoundCueQualityIndex))
	{
		OutOverrides.SoundCueCookQualityIndex = SoundCueQualityIndex;
	}

	if (bool bInlineFirstAudioChunk = false; PlatformFile->GetBool(*CategoryName, TEXT("bInlineFirstAudioChunk"), bInlineFirstAudioChunk))
	{
		OutOverrides.bInlineFirstAudioChunk = bInlineFirstAudioChunk;
	}

	if (float LengthOfFirstAudioChunkInSeconds = 0.f; PlatformFile->GetFloat(*CategoryName, TEXT("LengthOfFirstAudioChunkInSeconds"), LengthOfFirstAudioChunkInSeconds))
	{
		OutOverrides.LengthOfFirstAudioChunkInSecs = LengthOfFirstAudioChunkInSeconds;
	}

	/** Memory Load On Demand Settings */
	// Cache size:
	constexpr int32 DefaultCacheSizeKB = FAudioStreamCachingSettings::DefaultCacheSize;
	int32 RetrievedCacheSize = DefaultCacheSizeKB;
	if (PlatformFile->GetInt(*CategoryName, TEXT("CacheSizeKB"), RetrievedCacheSize))
	{
		if (RetrievedCacheSize == 0)
		{
			UE_LOGF(LogConfig, Display, "Audio Stream Cache \"Max Cache Size KB\" set to 0 by config: \"%ls%ls.ini\". Default value of %d KB will be used. You can update Project Settings here: Project Settings->Platforms->%ls->Audio->Cook Overrides->Stream Caching->Max Cache Size (KB)",
				PlatformFile->Branch ? *PlatformFile->Branch->SourceProjectConfigDir : TEXT(""), *PlatformFile->Name.ToString(), DefaultCacheSizeKB, *PlatformFile->PlatformName);
			RetrievedCacheSize = DefaultCacheSizeKB;
		}
	}

	OutOverrides.StreamCachingSettings.CacheSizeKB = RetrievedCacheSize;

	int32 RetrievedChunkSizeOverride = INDEX_NONE;
	if (PlatformFile->GetInt(*CategoryName, TEXT("MaxChunkSizeOverrideKB"), RetrievedChunkSizeOverride))
	{
		OutOverrides.StreamCachingSettings.MaxChunkSizeOverrideKB = RetrievedChunkSizeOverride;
	}

	bool bForceLegacyStreamChunking = false;
	if (PlatformFile->GetBool(*CategoryName, TEXT("bForceLegacyStreamChunking"), bForceLegacyStreamChunking))
	{
		OutOverrides.StreamCachingSettings.bForceLegacyStreamChunking = bForceLegacyStreamChunking;
	}

	int32 ZerothChunkSizeForLegacyStreamChunking = 0;
	if (PlatformFile->GetInt(*CategoryName, TEXT("ZerothChunkSizeForLegacyStreamChunking"), ZerothChunkSizeForLegacyStreamChunking))
	{
		OutOverrides.StreamCachingSettings.ZerothChunkSizeForLegacyStreamChunkingKB = ZerothChunkSizeForLegacyStreamChunking;
	}

	bool bResampleForDevice = false;
	if (PlatformFile->GetBool(*CategoryName, TEXT("bResampleForDevice"), bResampleForDevice))
	{
		OutOverrides.bResampleForDevice = bResampleForDevice;
	}

	float CompressionQualityModifier = 0.0f;
	if (PlatformFile->GetFloat(*CategoryName, TEXT("CompressionQualityModifier"), CompressionQualityModifier))
	{
		OutOverrides.CompressionQualityModifier = CompressionQualityModifier;
	}

	float AutoStreamingThreshold = 0.0f;
	if (PlatformFile->GetFloat(*CategoryName, TEXT("AutoStreamingThreshold"), AutoStreamingThreshold))
	{
		OutOverrides.AutoStreamingThreshold = AutoStreamingThreshold;
	}

#if 1
	//Cache sample rate map:
	float RetrievedSampleRate = -1.0f;

	if (PlatformFile->GetFloat(*CategoryName, TEXT("MaxSampleRate"), RetrievedSampleRate))
	{
		OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Max, RetrievedSampleRate);
	}

	RetrievedSampleRate = -1.0f;

	if (PlatformFile->GetFloat(*CategoryName, TEXT("HighSampleRate"), RetrievedSampleRate))
	{
		OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::High, RetrievedSampleRate);
	}


	RetrievedSampleRate = -1.0f;

	if (PlatformFile->GetFloat(*CategoryName, TEXT("MedSampleRate"), RetrievedSampleRate))
	{
		OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Medium, RetrievedSampleRate);
	}

	RetrievedSampleRate = -1.0f;

	if (PlatformFile->GetFloat(*CategoryName, TEXT("LowSampleRate"), RetrievedSampleRate))
	{
		OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Low, RetrievedSampleRate);
	}

	RetrievedSampleRate = -1.0f;

	if (PlatformFile->GetFloat(*CategoryName, TEXT("MinSampleRate"), RetrievedSampleRate))
	{
		OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Min, RetrievedSampleRate);
	}
#else

	//Cache sample rate map.
	OutOverrides.PlatformSampleRates.Reset();

	float RetrievedSampleRate = -1.0f;

	PlatformFile.GetFloat(*CategoryName, TEXT("MaxSampleRate"), RetrievedSampleRate);
	OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Max, RetrievedSampleRate);

	RetrievedSampleRate = -1.0f;

	PlatformFile.GetFloat(*CategoryName, TEXT("HighSampleRate"), RetrievedSampleRate);
	OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::High, RetrievedSampleRate);

	RetrievedSampleRate = -1.0f;

	PlatformFile.GetFloat(*CategoryName, TEXT("MedSampleRate"), RetrievedSampleRate);
	OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Medium, RetrievedSampleRate);

	RetrievedSampleRate = -1.0f;

	PlatformFile.GetFloat(*CategoryName, TEXT("LowSampleRate"), RetrievedSampleRate);
	OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Low, RetrievedSampleRate);

	RetrievedSampleRate = -1.0f;

	PlatformFile.GetFloat(*CategoryName, TEXT("MinSampleRate"), RetrievedSampleRate);
	OutOverrides.PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Min, RetrievedSampleRate);
#endif
}


static bool PlatformSupportsCompressionOverrides(const FString& PlatformName)
{
	return FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName).TargetSettingsIniSectionName.Len() > 0;
}

static inline FString GetCookOverridePlatformName(const TCHAR* PlatformName)
{
	return PlatformName ? FString(PlatformName) : FString(FPlatformProperties::IniPlatformName());
}

static bool PlatformSupportsCompressionOverrides(const TCHAR* PlatformName=nullptr)
{
	return PlatformSupportsCompressionOverrides(GetCookOverridePlatformName(PlatformName));
}

static FCriticalSection CookOverridesCriticalSection;

static FPlatformAudioCookOverrides& GetCacheableOverridesByPlatform(const TCHAR* InPlatformName, bool& bNeedsToBeInitialized)
{
	FScopeLock ScopeLock(&CookOverridesCriticalSection);

	// registry of overrides by platform name, for cooking, etc that may need multiple platforms worth
	static TMap<FString, FPlatformAudioCookOverrides> OverridesByPlatform;

	// make sure we don't reallocate the memory later
	if (OverridesByPlatform.Num() == 0)
	{
		// give enough space for all known platforms
		OverridesByPlatform.Reserve(FDataDrivenPlatformInfoRegistry::GetNumDataDrivenIniFiles());
	}

	FString PlatformName = GetCookOverridePlatformName(InPlatformName);
	// return one, or make one
	FPlatformAudioCookOverrides* ExistingOverrides = OverridesByPlatform.Find(PlatformName);
	if (ExistingOverrides != nullptr)
	{
		bNeedsToBeInitialized = false;
		return *ExistingOverrides;
	}

	bNeedsToBeInitialized = true;
	return OverridesByPlatform.Add(PlatformName, FPlatformAudioCookOverrides());
}

void FPlatformCompressionUtilities::RecacheCookOverrides()
{
	if (PlatformSupportsCompressionOverrides())
	{
		FScopeLock ScopeLock(&CookOverridesCriticalSection);
		bool bNeedsToBeInitialized;
		CacheAudioCookOverrides(GetCacheableOverridesByPlatform(nullptr, bNeedsToBeInitialized));
	}
}

const FPlatformAudioCookOverrides* FPlatformCompressionUtilities::GetCookOverrides(const TCHAR* PlatformName, bool bForceRecache)
{
	bool bNeedsToBeInitialized;
	FPlatformAudioCookOverrides& Overrides = GetCacheableOverridesByPlatform(PlatformName, bNeedsToBeInitialized);

#if WITH_EDITOR
	// In editor situations, the settings can change at any time, so we need to retrieve them.

	if (GIsEditor && !IsRunningCommandlet())
	{
		static TSAN_ATOMIC(double) LastCacheTime = 0.0;
		double CurrentTime = FPlatformTime::Seconds();
		double TimeSinceLastCache = CurrentTime - LastCacheTime;

		if (bForceRecache || TimeSinceLastCache > CookOverrideCachingIntervalCvar)
		{
			bNeedsToBeInitialized = true;
			LastCacheTime = CurrentTime;
		}
	}
#endif
	
	if (bNeedsToBeInitialized)
	{
		FScopeLock ScopeLock(&CookOverridesCriticalSection);
		CacheAudioCookOverrides(Overrides, PlatformName);
	}

	return &Overrides;
}

const FAudioStreamCachingSettings& FPlatformCompressionUtilities::GetStreamCachingSettingsForCurrentPlatform()
{
	const FPlatformAudioCookOverrides* Settings = GetCookOverrides();
	checkf(Settings, TEXT("Please only use this function if FPlatformCompressionUtilities::IsCurrentPlatformUsingLoadOnDemand() returns true."));
	return Settings->StreamCachingSettings;
}

FCachedAudioStreamingManagerParams FPlatformCompressionUtilities::BuildCachedStreamingManagerParams()
{
	const FAudioStreamCachingSettings& CacheSettings = GetStreamCachingSettingsForCurrentPlatform();
	int32 MaxChunkSize = GetMaxChunkSizeForCookOverrides(GetCookOverrides());

	const int32 MaxChunkSizeOverrideBytes = CacheSettings.MaxChunkSizeOverrideKB * 1024;
	if (MaxChunkSizeOverrideBytes > 0)
	{
		MaxChunkSize = FMath::Min(MaxChunkSizeOverrideBytes, MaxChunkSize);
	}

	// Our number of elements is tweakable based on the minimum cache usage we want to support.
	const float MinimumCacheUsage = FMath::Clamp(MinimumCacheUsageCvar, 0.0f, (1.0f - UE_KINDA_SMALL_NUMBER));
	int32 MinChunkSize = (1.0f - MinimumCacheUsage) * MaxChunkSize;
	
	uint64 TempNumElements = ((CacheSettings.CacheSizeKB * 1024) / MinChunkSize) * FMath::Max(ChunkSlotNumScalarCvar, 1.0f);
	int32 NumElements = FMath::Min(TempNumElements, static_cast<uint64>(TNumericLimits< int32 >::Max()));

	FCachedAudioStreamingManagerParams Params;
	FCachedAudioStreamingManagerParams::FCacheDimensions CacheDimensions;

	// Primary cache defined here:
	CacheDimensions.MaxChunkSize = 256 * 1024; // max possible chunk size (hard coded for legacy streaming path)
	CacheDimensions.MaxMemoryInBytes = CacheSettings.CacheSizeKB > 0 ? CacheSettings.CacheSizeKB * 1024 : FAudioStreamCachingSettings::DefaultCacheSize * 1024;
	CacheDimensions.NumElements = FMath::Max(NumElements, 1); // force at least a single cache element to avoid crashes
	Params.Caches.Add(CacheDimensions);

	// TODO: When settings are added to support multiple sub-caches, add it here.

	return Params;
}

uint32 FPlatformCompressionUtilities::GetMaxChunkSizeForCookOverrides(const FPlatformAudioCookOverrides* InCompressionOverrides)
{
	check(InCompressionOverrides);

	// TODO: We can fine tune this to platform-specific voice counts, but in the meantime we target 32 voices as an average-case.
	// If the game runs with higher than 32 voices, that means we will potentially have a larger cache than what was set in the target settings.
	// In that case we log a warning on application launch.
	const int32 MinimumNumChunks = 32;
	int32 CacheSizeKB = InCompressionOverrides->StreamCachingSettings.CacheSizeKB;

	const int32 DefaultMaxChunkSizeKB = 256;
	const int32 MaxChunkSizeOverrideKB = InCompressionOverrides->StreamCachingSettings.MaxChunkSizeOverrideKB;
	int32 ChunkSizeBasedOnUtilization = 0;

	if (CacheSizeKB / DefaultMaxChunkSizeKB < MinimumNumChunks)
	{
		ChunkSizeBasedOnUtilization = (CacheSizeKB / MinimumNumChunks);
	}

	return FMath::Max(FMath::Max(DefaultMaxChunkSizeKB, MaxChunkSizeOverrideKB), ChunkSizeBasedOnUtilization) * 1024;
}

float FPlatformCompressionUtilities::GetCompressionDurationForCurrentPlatform()
{
	float Threshold = -1.0f;

	const FPlatformRuntimeAudioCompressionOverrides* Settings = GetRuntimeCompressionOverridesForCurrentPlatform();
	if (Settings && Settings->bOverrideCompressionTimes)
	{
		Threshold = Settings->DurationThreshold;
	}

	return Threshold;
}

float FPlatformCompressionUtilities::GetTargetSampleRateForPlatform(ESoundwaveSampleRateSettings InSampleRateLevel /*= ESoundwaveSampleRateSettings::High*/)
{
	float SampleRate = -1.0f;
	const FPlatformAudioCookOverrides* Settings = GetCookOverrides();
	if (Settings && Settings->bResampleForDevice)
	{
		const float* FoundSampleRate = Settings->PlatformSampleRates.Find(InSampleRateLevel);

		if (FoundSampleRate)
		{
			SampleRate = *FoundSampleRate;
		}
		else
		{
			ensureMsgf(false, TEXT("Warning: Could not find a matching sample rate for this platform. Check your project settings."));
		}
	}

	return SampleRate;
}

int32 FPlatformCompressionUtilities::GetMaxPreloadedBranchesForCurrentPlatform()
{
	const FPlatformRuntimeAudioCompressionOverrides* Settings = GetRuntimeCompressionOverridesForCurrentPlatform();

	if (Settings)
	{
		return FMath::Max(Settings->MaxNumRandomBranches, 0);
	}
	else
	{
		return 0;
	}
}

int32 FPlatformCompressionUtilities::GetQualityIndexOverrideForCurrentPlatform()
{
	const FPlatformRuntimeAudioCompressionOverrides* Settings = GetRuntimeCompressionOverridesForCurrentPlatform();

	if (Settings)
	{
		return Settings->SoundCueQualityIndex;
	}
	else
	{
		return INDEX_NONE;
	}
}
