// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/AudioDebug.h"

#include "AudioDevice.h"
#include "AudioEffect.h"
#include "CanvasTypes.h"
#include "DrawDebugHelpers.h"
#include "Engine/Font.h"
#include "Engine/GameViewportClient.h"
#include "Sound/AudioSettings.h"
#include "Sound/ReverbEffect.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundSubmix.h"
#include "Stats/StatsTrace.h"
#include "UObject/UObjectIterator.h"
#include "UnrealEngine.h"

#if WITH_EDITOR
#include "Engine/GameViewportClient.h"
#include "LevelEditorViewport.h"
#else
#include "UnrealClient.h"
#endif // WITH_EDITOR


#ifndef ENABLE_AUDIO_DEBUG
#error "Please define ENABLE_AUDIO_DEBUG"
#endif //ENABLE_AUDIO_DEBUG

#if ENABLE_AUDIO_DEBUG

namespace Audio
{
	FString ExtraTracingSoundWaveFilterCVar;

#if !UE_BUILD_SHIPPING
	bool MatchesLogFilter(const TCHAR* InName)
	{
		return !ExtraTracingSoundWaveFilterCVar.IsEmpty()	// Trivially reject if we have no filter.
		 && FCString::Stristr(InName, *ExtraTracingSoundWaveFilterCVar);
	}
#endif
}

static FAutoConsoleVariableRef CVarExtraTracingSoundWaveFilter(
	TEXT("au.debug.extra_logging_filter"),
	Audio::ExtraTracingSoundWaveFilterCVar,
	TEXT("Enable extra logging for any sound wave that has this substring in its name."),
	ECVF_Default);

// Console variables
static int32 ActiveSoundVisualizeModeCVar = 1;
FAutoConsoleVariableRef CVarAudioVisualizeActiveSoundsMode(
	TEXT("au.3dVisualize.ActiveSounds"),
	ActiveSoundVisualizeModeCVar,
	TEXT("Visualization mode for active sounds. \n")
	TEXT("0: Not Enabled, 1: Volume (Lin), 2: Volume (dB), 3: Distance, 4: Random color, 5: Occlusion"),
	ECVF_Default);

static int32 ActiveSoundVisualizeListenersCVar = 0;
FAutoConsoleVariableRef CVarAudioVisualizeListeners(
	TEXT("au.3dVisualize.Listeners"),
	ActiveSoundVisualizeListenersCVar,
	TEXT("Whether or not listeners are visible when 3d visualize is enabled. \n")
	TEXT("0: Not Enabled, 1: Enabled"), 
	ECVF_Default);

static int32 ActiveSoundVisualizeTypeCVar = 0;
FAutoConsoleVariableRef CVarAudioVisualizeActiveSounds(
	TEXT("au.3dVisualize.ActiveSounds.Type"),
	ActiveSoundVisualizeTypeCVar,
	TEXT("Whether to show all sounds, on AudioComponents (Components Only), or off of AudioComponents (Non-Component Only). \n")
	TEXT("0: All, 1: Components Only, 2: Non-Component Only"),
	ECVF_Default);

static int32 SpatialSourceVisualizeEnabledCVar = 1;
FAutoConsoleVariableRef CVarAudioVisualizeSpatialSourceEnabled(
	TEXT("au.3dVisualize.SpatialSources"),
	SpatialSourceVisualizeEnabledCVar,
	TEXT("Whether or not audio spatialized sources are visible when 3d visualize is enabled. \n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 VirtualLoopsVisualizeEnabledCVar = 1;
FAutoConsoleVariableRef CVarAudioVisualizeVirtualLoopsEnabled(
	TEXT("au.3dVisualize.VirtualLoops"),
	VirtualLoopsVisualizeEnabledCVar,
	TEXT("Whether or not virtualized loops are visible when 3d visualize is enabled. \n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 AudioDebugSoundMaxNumDisplayedCVar = 32;
FAutoConsoleVariableRef CVarAudioDebugSoundMaxNumDisplayed(
	TEXT("au.Debug.Sounds.Max"),
	AudioDebugSoundMaxNumDisplayedCVar,
	TEXT("Max number of sounds to display in full sound debugger view. \n")
	TEXT("Default: 32"),
	ECVF_Default);

static int32 AudioDebugSoundShowPathCVar = 1;
FAutoConsoleVariableRef CVarAudioDebugSoundShowPath(
	TEXT("au.Debug.Sounds.ShowPath"),
	AudioDebugSoundShowPathCVar,
	TEXT("Display full path of sound when enabled.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static FString AudioDebugSoundSortCVarCVar = "Name";
FAutoConsoleVariableRef CVarAudioDebugSoundSortCVar(
	TEXT("au.Debug.Sounds.Sort"),
	AudioDebugSoundSortCVarCVar,
	TEXT("Value to sort by and display when sound stats are active. \n")
	TEXT("Class, Distance, Name (Default), Priority (Highest of wave instances per sound), Time, Waves, Volume"),
	ECVF_Default);

static FString AudioDebugStatSoundTextColorCVar = "White";
FAutoConsoleVariableRef CVarAudioDebugStatSoundColor(
	TEXT("au.Debug.Sounds.TextColor"),
	AudioDebugStatSoundTextColorCVar,
	TEXT("Color of body text in audio debug views. \n")
	TEXT("White, Red, Orange, Yellow, Blue, Magenta, Purple, Black"),
	ECVF_Default);

static int32 SoundCueDebugShowPathCVar = 1;
FAutoConsoleVariableRef CVarAudioSoundCueDebugShowPath(
	TEXT("au.Debug.Soundcues.ShowPath"),
	SoundCueDebugShowPathCVar,
	TEXT("Display full path of sound cue when enabled.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 SoundCueDebugShowDistanceCVar = 0;
FAutoConsoleVariableRef CVarAudioSoundCueDebugShowDistance(
	TEXT("au.Debug.Soundcues.ShowDistance"),
	SoundCueDebugShowDistanceCVar,
	TEXT("Display distance of sound cue when enabled.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 SoundCueDebugMinimalCVar = 0;
FAutoConsoleVariableRef CVarAudioSoundCueDebugMinimal(
	TEXT("au.Debug.SoundCues.Minimal"),
	SoundCueDebugMinimalCVar,
	TEXT("Use the compact view of sound cue debug when enabled. \n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 SoundCueDebugTabSpacingCVar = 5;
FAutoConsoleVariableRef CVarAudioSoundCueDebugTabSpacing(
	TEXT("au.Debug.SoundCues.Spacing.Tab"),
	SoundCueDebugTabSpacingCVar,
	TEXT("Size of tab (in characters) with compact view. \n")
	TEXT("Default: 5"),
	ECVF_Default);

static int32 SoundCueDebugCharSpacingCVar = 7;
FAutoConsoleVariableRef CVarSoundCueDebugCharSpacing(
	TEXT("au.Debug.SoundCues.Spacing.Char"),
	SoundCueDebugCharSpacingCVar,
	TEXT("Size of character (in pixels) with compact view. \n")
	TEXT("Default: 7"),
	ECVF_Default);

static int32 SoundDebugDisplayCornerXCVar = 100;
FAutoConsoleVariableRef CVarSoundDebugDisplayCornerX(
	TEXT("au.Debug.Display.X"),
	SoundDebugDisplayCornerXCVar,
	TEXT("X position on screen of debug statistics. \n")
	TEXT("Default: 100"),
	ECVF_Default);

static int32 SoundDebugDisplayCornerYCVar = -1;
FAutoConsoleVariableRef CVarSoundDebugDisplayCornerY(
	TEXT("au.Debug.Display.Y"),
	SoundDebugDisplayCornerYCVar,
	TEXT("X position on screen of debug statistics. \n")
	TEXT("Default: -1 (Disabled, uses default debug position)"),
	ECVF_Default);

namespace Audio
{
	const FColor HeaderColor = FColor::Green;

	FColor GetBodyColor()
	{
		return FColor::White;
	}

	FColor GetStatSoundBodyColor()
	{
		if (AudioDebugStatSoundTextColorCVar == TEXT("Red"))
		{
			return FColor::Red;
		}
		if (AudioDebugStatSoundTextColorCVar == TEXT("Orange"))
		{
			return FColor::Orange;
		}
		if (AudioDebugStatSoundTextColorCVar == TEXT("Yellow"))
		{
			return FColor::Yellow;
		}
		if (AudioDebugStatSoundTextColorCVar == TEXT("Green"))
		{
			return FColor::Green;
		}
		if (AudioDebugStatSoundTextColorCVar == TEXT("Blue"))
		{
			return FColor::Blue;
		}
		if (AudioDebugStatSoundTextColorCVar == TEXT("Magenta"))
		{
			return FColor::Magenta;
		}
		if (AudioDebugStatSoundTextColorCVar == TEXT("Purple"))
		{
			return FColor::Purple;
		}
		if (AudioDebugStatSoundTextColorCVar == TEXT("Black"))
		{
			return FColor::Black;
		}

		return FColor::White;
	}

	// Whether or not respective stat data is active for any audio device (set on game thread)
	static bool bDebugWavesForAllViewsEnabled = false;
	static bool bDebugCuesForAllViewsEnabled = false;
	static bool bDebugSoundsForAllViewsEnabled = false;
	static bool bDebugMixesForAllViewsEnabled = false;
	static bool bDebugReverbForAllViewsEnabled = false;
	static bool bDebugModulationForAllViewsEnabled = false;
	static bool bDebugStreamingForAllViewsEnabled = false;

	const int32 DebuggerTabWidth = 12;

	const float MinDisplayVolume = UE_KINDA_SMALL_NUMBER; // -80 dB

	FAudioDevice* GetWorldAudio(UWorld* World)
	{
		check(IsInGameThread());

		if (!World)
		{
			return nullptr;
		}

		return World->GetAudioDeviceRaw();
	}

	namespace DebugStatNames
	{
		const FName SoundWaves = "SoundWaves";
		const FName SoundCues = "SoundCues";
		const FName Sounds = "Sounds";
		const FName SoundMixes = "SoundMixes";
		const FName SoundModulation = "SoundModulation";
		const FName SoundReverb = "SoundReverb";
		const FName AudioStreaming = "AudioStreaming";

		// TODO: Move to console variables
		const FName DebugSounds = "DebugSounds";
		const FName LongSoundNames = "LogSoundNames";
	}

	struct FAudioStats
	{
		enum class EDisplaySort : uint8
		{
			Class,
			Distance,
			Name,
			PlaybackTime,
			Priority,
			Waves,
			Volume
		};

		enum class EDisplayFlags : uint8
		{
			Debug = 0x01,
			Long_Names = 0x40,
		};

		struct FStatWaveInstanceInfo
		{
			TSharedPtr<FSoundSource::FDebugInfo, ESPMode::ThreadSafe> DebugInfo;
			FString Description;
			float Volume;
			int32 InstanceIndex;
			FName WaveInstanceName;
			FName SoundClassName;
			uint8 bPlayWhenSilent : 1;
		};

		struct FStatSoundInfo
		{
			FString SoundName;
			FString SoundPath;
			FName SoundClassName;
			float Distance;
			float PlaybackTime;
			float PlaybackTimeNonVirtualized;
			float Priority;
			float Volume;
			uint32 AudioComponentID;
			FTransform Transform;
			TArray<FStatWaveInstanceInfo> WaveInstanceInfos;

			TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails> ShapeDetailsMap;
		};

		struct FStatSoundMix
		{
			FString MixName;
			float InterpValue;
			int32 RefCount;
			bool bIsCurrentEQ;
		};

		uint8 DisplayFlags;
		EDisplaySort DisplaySort;
		TSet<FName> EnabledStats;
		TArray<FTransform> ListenerTransforms;
		TArray<FStatSoundInfo> StatSoundInfos;
		TArray<FStatSoundMix> StatSoundMixes;

		FAudioStats()
			: DisplayFlags(0)
			, DisplaySort(EDisplaySort::Name)
			, EnabledStats()
		{
		}
	};

	void UpdateDisplaySort(FAudioStats& InAudioStats)
	{
		if (AudioDebugSoundSortCVarCVar == TEXT("distance"))
		{
			InAudioStats.DisplaySort = FAudioStats::EDisplaySort::Distance;
		}
		else if (AudioDebugSoundSortCVarCVar == TEXT("class"))
		{
			InAudioStats.DisplaySort = FAudioStats::EDisplaySort::Class;
		}
		else if (AudioDebugSoundSortCVarCVar == TEXT("name"))
		{
			InAudioStats.DisplaySort = FAudioStats::EDisplaySort::Name;
		}
		else if (AudioDebugSoundSortCVarCVar == TEXT("time"))
		{
			InAudioStats.DisplaySort = FAudioStats::EDisplaySort::PlaybackTime;
		}
		else if (AudioDebugSoundSortCVarCVar == TEXT("priority"))
		{
			InAudioStats.DisplaySort = FAudioStats::EDisplaySort::Priority;
		}
		else if (AudioDebugSoundSortCVarCVar == TEXT("volume"))
		{
			InAudioStats.DisplaySort = FAudioStats::EDisplaySort::Volume;
		}
		else if (AudioDebugSoundSortCVarCVar == TEXT("waves"))
		{
			InAudioStats.DisplaySort = FAudioStats::EDisplaySort::Waves;
		}
	}

	struct FAudioStats_AudioThread
	{
		TSet<FName> RequestedStats;

		FAudioStats_AudioThread()
			: RequestedStats()
		{
		}
	};

	TMap<FDeviceId, FAudioStats> AudioDeviceStats;
	TMap<FDeviceId, FAudioStats_AudioThread> AudioDeviceStats_AudioThread;

	void HandleDumpActiveSounds(UWorld* World)
	{
		if (GEngine && GEngine->GetAudioDeviceManager())
		{
			GEngine->GetAudioDeviceManager()->GetDebugger().DumpActiveSounds();
		}
	}

	void HandleClearMutesAndSolos(UWorld* World)
	{
		if (GEngine && GEngine->GetAudioDeviceManager())
		{
			GEngine->GetAudioDeviceManager()->GetDebugger().ClearMutesAndSolos();
		}
	}

	template <typename SoundType>
	void PlayDebugSound(const TCHAR* Cmd, FAudioDevice& InAudioDevice, UWorld& InWorld, TUniqueFunction<void(SoundType& /*InSound*/)> InInitFunction)
	{
		UAudioComponent* TestComp = InAudioDevice.GetTestComponent(&InWorld);
		if (!TestComp)
		{
			return;
		}

		bool bAssetPathSet = false;
		FString AssetPath;
		if (FParse::Value(Cmd, TEXT("Name"), AssetPath))
		{
			if (const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>())
			{
				const FName SoundName(*AssetPath);
				for (const FSoundDebugEntry& DebugSound : AudioSettings->DebugSounds)
				{
					const FSoftObjectPath& ObjectPath = DebugSound.Sound;
					if (DebugSound.DebugName == SoundName && ObjectPath.IsValid())
					{
						AssetPath = ObjectPath.ToString();
						bAssetPathSet = true;
						break;
					}
				}
			}
		}

		if (!bAssetPathSet)
		{
			bAssetPathSet = FParse::Value(Cmd, TEXT("Path"), AssetPath);
		}

		if (!bAssetPathSet)
		{
			TArray<FString> Tokens;
			FString CmdStr = Cmd;
			CmdStr.ParseIntoArray(Tokens, TEXT(" "));
			if (!Tokens.Num())
			{
				return;
			}

			AssetPath = Tokens[0];
		}

		// Load up an arbitrary cue
		SoundType* SoundObject = LoadObject<SoundType>(nullptr, *AssetPath, nullptr, LOAD_None, nullptr);
		if (!SoundObject)
		{
			return;
		}

		TestComp->Stop();
		TestComp->Sound = SoundObject;
		TestComp->bAutoDestroy = true;

		float Radius = 1.0f;
		float Azimuth = 0.0f;
		float Elevation = 0.0f;
		TestComp->bAllowSpatialization |= FParse::Value(Cmd, TEXT("Radius"), Radius);
		TestComp->bAllowSpatialization |= FParse::Value(Cmd, TEXT("Azimuth"), Azimuth);
		TestComp->bAllowSpatialization |= FParse::Value(Cmd, TEXT("Elevation"), Elevation);
		if (TestComp->bAllowSpatialization)
		{
			TestComp->bAllowSpatialization = true;
			FTransform TestTransform;
			InAudioDevice.GetListenerTransform(0, TestTransform);

			static const float AziOffset = 90.0f;
			static const float ElevOffset = 90.0f;

			const FVector EulerAngles = TestTransform.GetRotation().Euler();
			Azimuth = FMath::DegreesToRadians(-1.0f * (EulerAngles.Z + Azimuth + AziOffset));
			Elevation = FMath::DegreesToRadians(Elevation + EulerAngles.Y - ElevOffset);

			const float X = Radius * FMath::Sin(Elevation) * FMath::Sin(Azimuth);
			const float Y = Radius * FMath::Sin(Elevation) * FMath::Cos(Azimuth);
			const float Z = Radius * FMath::Cos(Elevation);

			const FVector Translation(X, Y, Z);
			TestTransform.AddToTranslation(Translation);
			TestComp->SetComponentToWorld(TestTransform);
		}

		InInitFunction(*SoundObject);
		TestComp->Play();
	}

	void HandlePlayDebugSoundCue(const TArray<FString>& InArgs, UWorld* World)
	{
		if (!World)
		{
			return;
		}

		auto PlayDebugSoundCue = [](const TCHAR * InCmd, FAudioDevice & InAudioDevice, UWorld & InWorld)
		{
			bool bSetLooping = false;
			PlayDebugSound<USoundCue>(InCmd, InAudioDevice, InWorld, [InCmd](USoundCue& InCue)
			{
				TArray<USoundNodeWavePlayer*> WavePlayers;
				InCue.RecursiveFindNode<USoundNodeWavePlayer>(InCue.FirstNode, WavePlayers);

				for (int32 i = 0; i < WavePlayers.Num(); ++i)
				{
					if (USoundWave* SoundWave = WavePlayers[i]->GetSoundWave())
					{
						FAudioDebugger::LogSubtitle(InCmd, *SoundWave);
					}
				}
			});
		};

		FString Cmd = FString::Join(InArgs, TEXT(" "));

		if (FParse::Param(*Cmd, TEXT("AllViews")))
		{
			if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
			{
				DeviceManager->IterateOverAllDevices([PlayDebugSoundCue, Cmd, World](Audio::FDeviceId, FAudioDevice* AudioDevice)
				{
					if (AudioDevice)
					{
						PlayDebugSoundCue(*Cmd, *AudioDevice, *World);
					}
				});
			}
		}
		else
		{
			if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
			{
				PlayDebugSoundCue(*Cmd, *AudioDevice, *World);
			}
		}
	}

	void HandlePlayDebugSoundWave(const TArray<FString>& InArgs, UWorld* World)
	{
		if (!World)
		{
			return;
		}

		FString Cmd = FString::Join(InArgs, TEXT(" "));

		auto PlayDebugSoundWave = [](const TCHAR * InCmd, FAudioDevice & InAudioDevice, UWorld & InWorld)
		{
			PlayDebugSound<USoundWave>(InCmd, InAudioDevice, InWorld, [InCmd](USoundWave& InSoundWave)
			{
				FAudioDebugger::LogSubtitle(InCmd, InSoundWave);
			});
		};

		if (FParse::Param(*Cmd, TEXT("AllViews")))
		{
			if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
			{
				DeviceManager->IterateOverAllDevices([PlayDebugSoundWave, Cmd, World](Audio::FDeviceId, FAudioDevice* AudioDevice)
				{
					if (AudioDevice)
					{
						PlayDebugSoundWave(*Cmd, *AudioDevice, *World);
					}
				});
			}
		}
		else if (World)
		{
			if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
			{
				PlayDebugSoundWave(*Cmd, *AudioDevice, *World);
			}
		}
	}

	void HandleStopDebugSound(const TArray<FString>& InArgs, UWorld* InWorld)
	{
		if (!InWorld)
		{
			return;
		}

		FString Cmd = FString::Join(InArgs, TEXT(" "));

		if (FParse::Param(*Cmd, TEXT("AllViews")))
		{
			if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
			{
				DeviceManager->IterateOverAllDevices([InWorld](Audio::FDeviceId, FAudioDevice* AudioDevice)
				{
					if (AudioDevice)
					{
						AudioDevice->StopTestComponent();
					}
				});
			}
		}
		else if (InWorld)
		{
			if (FAudioDevice* AudioDevice = InWorld->GetAudioDeviceRaw())
			{
				AudioDevice->StopTestComponent();
			}
		}
	}

	void DebugSoundObject(const TArray<FString>& Args, UWorld* InWorld, const FName InStatToEnable, bool& bAllEnabled)
	{
		if (Args.Num() > 0)
		{
			bool bAllViews = false;
			for (int32 i = 1; i < Args.Num(); ++i)
			{
				bAllViews |= Args[i] == TEXT("-AllViews");
			}

			bool bEnablementRequest = Args[0].ToBool();
			bAllEnabled = bAllViews ? bEnablementRequest : false;

			TSet<FName> Stats;
			Stats.Add(InStatToEnable);

			bEnablementRequest
				? Audio::FAudioDebugger::SetStats(Stats, bAllViews ? nullptr : InWorld)
				: Audio::FAudioDebugger::ClearStats(InStatToEnable, bAllViews ? nullptr : InWorld);
		}
	}

	bool DebugShouldRenderStat(UWorld* World, FCanvas* Canvas, bool bEnablementBool, const FName InAudioStat, FAudioDevice** OutAudioDevice)
	{
		if (!Canvas || !World)
		{
			return false;
		}

		*OutAudioDevice = World->GetAudioDeviceRaw();
		if (!*OutAudioDevice)
		{
			return false;
		}

		if (bEnablementBool)
		{
			return true;
		}

		if (!bEnablementBool)
		{
			FAudioStats* AudioStats = AudioDeviceStats.Find((*OutAudioDevice)->DeviceID);
			if (!AudioStats   || !(AudioStats->EnabledStats.Contains(InAudioStat)) )
			{
				return false;
			}
		}

		return true;
	}

	TObjectPtr<USoundSubmix> FindSubmixByName(const FName InSubmixName)
	{
		for (TObjectIterator<USoundSubmix> It; It; ++It)
		{
			const TObjectPtr<USoundSubmix> Submix = *It;
			if (Submix && Submix->GetFName() == InSubmixName)
			{
				return Submix;
			}
		}

		return nullptr;
	}
} // namespace Audio

// Console Commands
static FAutoConsoleCommandWithWorld GAudioDumpActiveSounds(TEXT("au.DumpActiveSounds"), TEXT("Outputs data about all the currently active sounds."), FConsoleCommandWithWorldDelegate::CreateStatic(&Audio::HandleDumpActiveSounds), ECVF_Cheat);
static FAutoConsoleCommandWithWorld GAudioClearMutesAndSolos(TEXT("au.ClearMutesAndSolos"), TEXT("Clears any solo-ing/mute-ing sounds"), FConsoleCommandWithWorldDelegate::CreateStatic(&Audio::HandleClearMutesAndSolos), ECVF_Cheat);

static FAutoConsoleCommandWithWorldAndArgs GAudioPlayDebugSoundCue
(
	TEXT("au.Debug.PlaySoundCue"),
	TEXT("Plays a SoundCue:\n")
	TEXT("-Name <SoundName>: If a debug sound with the short name is specified in AudioSettings, plays that sound.\n")
	TEXT("-Path <ObjectPath>: Finds SoundCue asset at the provided path and if found, plays that sound.\n")
	TEXT("-Radius <Distance>: If set, enables sound spatialization and sets radial distance between listener and source emitting sound.\n")
	TEXT("-Azimuth <Angle>: If set, enables sound spatialization and sets azimuth angle between listener and source emitting sound (in degrees, where 0 is straight ahead, negative to left, positive to right).\n")
	TEXT("-Elevation <Angle>: If set, enables sound spatialization and sets azimuth angle between listener and source emitting sound (in degrees, where 0 is straight ahead, negative to left, positive to right).\n")
	TEXT("-AllViews: If option provided, plays sound through all viewports.\n")
	TEXT("-LogSubtitles: If option provided, logs sounds subtitle if set\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&Audio::HandlePlayDebugSoundCue),
	ECVF_Cheat
);

static FAutoConsoleCommandWithWorldAndArgs GAudioPlayDebugSoundWave
(
	TEXT("au.Debug.PlaySoundWave"),
	TEXT("Plays a SoundWave:\n")
		TEXT("-Name <SoundName>: If a debug sound with the short name is specified in AudioSettings, plays that sound.\n")
		TEXT("-Path <ObjectPath>: Finds SoundWave asset at the provided path and if found, plays that sound.\n")
		TEXT("-Radius: If set, enables sound spatialization and sets radial distance between listener and source emitting sound.\n")
		TEXT("-Azimuth <Angle>: If set, enables sound spatialization and sets azimuth angle between listener and source emitting sound (in degrees, where 0 is straight ahead, negative to left, positive to right).\n")
		TEXT("-Elevation <Angle>: If set, enables sound spatialization and sets azimuth angle between listener and source emitting sound (in degrees, where 0 is straight ahead, negative to left, positive to right).\n")
		TEXT("-AllViews: If option provided, plays sound through all viewports.\n")
		TEXT("-LogSubtitles: If option provided, logs sounds subtitle if set\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&Audio::HandlePlayDebugSoundWave),
	ECVF_Cheat
);

static FAutoConsoleCommandWithWorldAndArgs GAudioStopDebugSound
(
	TEXT("au.Debug.StopSound"),
	TEXT("Stops debug sound.\n")
		TEXT("-AllViews: If option provided, stops all debug sounds in all viewports.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&Audio::HandleStopDebugSound),
	ECVF_Cheat
);

static FAutoConsoleCommandWithWorldAndArgs GAudioDebugSoundCues
(
	TEXT("au.Debug.SoundCues"),
		TEXT("Post SoundCue information to viewport(s).\n")
		TEXT("0: Disable, 1: Enable\n")
	TEXT("(Optional) -AllViews: Enables/Disables for all viewports, not just those associated with the current world"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* InWorld)
	{
		Audio::DebugSoundObject(Args, InWorld, Audio::DebugStatNames::SoundCues, Audio::bDebugCuesForAllViewsEnabled);
	}), ECVF_Cheat
);

static FAutoConsoleCommandWithWorldAndArgs GAudioDebugSounds
(
	TEXT("au.Debug.Sounds"),
		TEXT("Post Sound information to viewport(s).\n")
		TEXT("0: Disable, 1: Enable\n")
	TEXT("(Optional) -AllViews: Enables/Disables for all viewports, not just those associated with the current world"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* InWorld)
	{
		Audio::DebugSoundObject(Args, InWorld, Audio::DebugStatNames::Sounds, Audio::bDebugSoundsForAllViewsEnabled);
	}), ECVF_Cheat
);

static bool bAttenuationVisualizeEnabledCVar = false;
FAutoConsoleCommandWithWorldAndArgs CVarAudioVisualizeAttenuation(
	TEXT("au.3dVisualize.Attenuation"),
	TEXT("Whether or not attenuation spheres are visible when 3d visualize is enabled. \n")
	TEXT("0: Not Enabled, 1: Enabled"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* InWorld)
	{
		if (Args.Num() <= 0)
		{
			return;
		}

		bAttenuationVisualizeEnabledCVar = Args[0].ToBool();

		// Internally, the visualization code requires au.Debug.Sounds to be enabled,
		// so we force it on whenever this one is enabled.
		if (bAttenuationVisualizeEnabledCVar)
		{
			Audio::DebugSoundObject(Args, InWorld, Audio::DebugStatNames::Sounds, Audio::bDebugSoundsForAllViewsEnabled);
		}
	}),
	ECVF_Default);

static FAutoConsoleCommandWithWorldAndArgs GAudioDebugSoundWaves
(
	TEXT("au.Debug.SoundWaves"),
	TEXT("Post SoundWave information to viewport(s).\n")
		TEXT("0: Disable, 1: Enable\n")
		TEXT("(Optional) -AllViews: Enables/Disables for all viewports, not just those associated with the current world"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* InWorld)
	{
		Audio::DebugSoundObject(Args, InWorld, Audio::DebugStatNames::SoundWaves, Audio::bDebugWavesForAllViewsEnabled);
	}), ECVF_Cheat
);

static FAutoConsoleCommandWithWorldAndArgs GAudioDebugSoundMixes
(
	TEXT("au.Debug.SoundMixes"),
	TEXT("Post SoundMix information to viewport(s).\n")
		TEXT("0: Disable, 1: Enable\n")
		TEXT("(Optional) -AllViews: Enables/Disables for all viewports, not just those associated with the current world"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* InWorld)
	{
		Audio::DebugSoundObject(Args, InWorld, Audio::DebugStatNames::SoundMixes, Audio::bDebugMixesForAllViewsEnabled);
	}), ECVF_Cheat
);

static FAutoConsoleCommandWithWorldAndArgs GAudioDebugSoundReverb
(
	TEXT("au.Debug.SoundReverb"),
	TEXT("Post SoundReverb information to viewport(s).\n")
	TEXT("0: Disable, 1: Enable\n")
	TEXT("(Optional) -AllViews: Enables/Disables for all viewports, not just those associated with the current world"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* InWorld)
	{
		Audio::DebugSoundObject(Args, InWorld, Audio::DebugStatNames::SoundReverb, Audio::bDebugReverbForAllViewsEnabled);
	}), ECVF_Cheat
);

static FAutoConsoleCommandWithWorldAndArgs GAudioDebugSoundModulation
(
	TEXT("au.Debug.Modulation"),
	TEXT("Post Audio Modulation information to viewport(s).\n")
		TEXT("0: Disable, 1: Enable\n")
		TEXT("(Optional) -AllViews: Enables/Disables for all viewports, not just those associated with the current world"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* InWorld)
	{
		Audio::DebugSoundObject(Args, InWorld, Audio::DebugStatNames::SoundModulation, Audio::bDebugModulationForAllViewsEnabled);
	}), ECVF_Cheat
);

static FAutoConsoleCommandWithWorldAndArgs GAudioDebugStreaming
(
	TEXT("au.Debug.Streaming"),
	TEXT("Post Stream Caching information to viewport(s).\n")
	TEXT("0: Disable, 1: Enable\n")
	TEXT("(Optional) -AllViews: Enables/Disables for all viewports, not just those associated with the current world"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* InWorld)
		{
			Audio::DebugSoundObject(Args, InWorld, Audio::DebugStatNames::AudioStreaming, Audio::bDebugStreamingForAllViewsEnabled);
		}), ECVF_Cheat
);

namespace Audio
{
	/** Audio Debugger Implementation */
	FAudioDebugger::FAudioDebugger()
		: bVisualize3dDebug(0)
	{
		WorldRegisteredWithDeviceHandle = FAudioDeviceWorldDelegates::OnWorldRegisteredToAudioDevice.AddLambda([this](const UWorld* InWorld, FDeviceId InDeviceId)
		{
			TSet<FName> StatsToSet;

			if (bDebugSoundsForAllViewsEnabled)
			{
				StatsToSet.Add(DebugStatNames::Sounds);
			}

			if (bDebugCuesForAllViewsEnabled)
			{
				StatsToSet.Add(DebugStatNames::SoundCues);
			}

			if (bDebugWavesForAllViewsEnabled)
			{
				StatsToSet.Add(DebugStatNames::SoundWaves);
			}

			if (bDebugModulationForAllViewsEnabled)
			{
				StatsToSet.Add(DebugStatNames::SoundModulation);
			}

			if (bDebugMixesForAllViewsEnabled)
			{
				StatsToSet.Add(DebugStatNames::SoundMixes);
			}

			if (bDebugReverbForAllViewsEnabled)
			{
				StatsToSet.Add(DebugStatNames::SoundReverb);
			}

			if (bDebugStreamingForAllViewsEnabled)
			{
				StatsToSet.Add(DebugStatNames::AudioStreaming);
			}

			SetStats(InDeviceId, StatsToSet);
		});
	}

	bool FAudioDebugger::IsVisualizeDebug3dEnabled() const
	{
		return bVisualize3dDebug;
	}

	void FAudioDebugger::ToggleVisualizeDebug3dEnabled()
	{
		bVisualize3dDebug = !bVisualize3dDebug;
	}

	bool FAudioDebugger::IsVirtualLoopVisualizeEnabled()
	{
		return static_cast<bool>(VirtualLoopsVisualizeEnabledCVar);
	}

	#if WITH_EDITOR
	void FAudioDebugger::OnBeginPIE()
	{
		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		if (DeviceManager)
		{
			DeviceManager->GetDebugger().ClearMutesAndSolos();
		}
	}

	void FAudioDebugger::OnEndPIE()
	{
		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		if (DeviceManager)
		{
			DeviceManager->GetDebugger().ClearMutesAndSolos();
		}
	}
	#endif // WITH_EDITOR

	void FAudioDebugger::QuerySoloMuteSoundClass(const FString& Name, bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const
	{
		GetDebugSoloMuteStateX(Name, DebugNames.SoloSoundClass, DebugNames.MuteSoundClass, bOutIsSoloed, bOutIsMuted, OutReason);
	}

	void FAudioDebugger::QuerySoloMuteSoundWave(const FString& Name, bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const
	{
		GetDebugSoloMuteStateX(Name, DebugNames.SoloSoundWave, DebugNames.MuteSoundWave, bOutIsSoloed, bOutIsMuted, OutReason);
	}

	void FAudioDebugger::QuerySoloMuteSoundCue(const FString& Name, bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const
	{
		GetDebugSoloMuteStateX(Name, DebugNames.SoloSoundCue, DebugNames.MuteSoundCue, bOutIsSoloed, bOutIsMuted, OutReason);
	}

	void FAudioDebugger::QuerySoloMuteSubmix(const FString& Name, bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const
	{
		GetDebugSoloMuteStateX(Name, DebugNames.SoloSubmix, DebugNames.MuteSubmix, bOutIsSoloed, bOutIsMuted, OutReason);
	}

	void FAudioDebugger::SetNameArray(FName InName, TArray<FName>& InNameArray, bool bOnOff)
	{
		ExecuteCmdOnAudioThread([InName, &InNameArray, bOnOff]
		{
			if (bOnOff)
			{
				InNameArray.AddUnique(InName);
			}
			else
			{
				InNameArray.Remove(InName);
			}
		});
	}

	void FAudioDebugger::ToggleNameArray(FName InName, TArray<FName>& InNameArray, bool bExclusive )
	{
		ExecuteCmdOnAudioThread([InName, &InNameArray, bExclusive]
		{ 
			// On already?
			int32 IndexOf = InNameArray.IndexOfByKey(InName);
			if (IndexOf != INDEX_NONE)
			{			
				if (bExclusive)
				{
					// Turn off everything if we are exclusive.
					InNameArray.Empty();
				}
				else
				{
					InNameArray.RemoveAtSwap(IndexOf);
				}
			}
			else // Add it.
			{
				// If we are exclusive, turn off everything else first.
				if (bExclusive)
				{
					InNameArray.Empty();
				}

				// Allow for NAME_None to act as a clear
				if (InName != NAME_None)
				{
					InNameArray.Add(InName);
				}						
			}
		});
	}

	void FAudioDebugger::SetAudioMixerDebugSound(const TCHAR* SoundName)
	{
		ExecuteCmdOnAudioThread([Name = FString(SoundName), this]{ DebugNames.DebugAudioMixerSoundName = Name; });
	}

	void FAudioDebugger::SetAudioDebugSound(const TCHAR* SoundName)
	{
		ExecuteCmdOnAudioThread([Name = FString(SoundName), this]{ 
			DebugNames.DebugSoundName = Name;
			DebugNames.bDebugSoundName = DebugNames.DebugSoundName != TEXT("");
		});	
	}

	const FString& FAudioDebugger::GetAudioMixerDebugSoundName() const
	{
		check(IsInAudioThread());
		return DebugNames.DebugAudioMixerSoundName;
	}

	bool FAudioDebugger::GetAudioDebugSound(FString& OutDebugSound)
	{
		check(IsInAudioThread());
		if (DebugNames.bDebugSoundName)
		{
			OutDebugSound = DebugNames.DebugSoundName;
			return true;
		}
		return false;
	}

	void FAudioDebugger::ExecuteCmdOnAudioThread(TFunction<void()> Cmd)
	{
		// If not on audio thread, queue it.
		if (!IsInAudioThread())
		{
			FAudioThread::RunCommandOnAudioThread(Cmd);
			return;
		}

		// Otherwise, do it inline.
		Cmd();
	}

	void FAudioDebugger::GetDebugSoloMuteStateX(
		const FString& Name, 
		const TArray<FName>& Solos,
		const TArray<FName>& Mutes,	
		bool& bOutIsSoloed, 
		bool& bOutIsMuted, 
		FString& OutReason) const
	{
		check(IsInAudioThread());

		const FString PlainNameString = FName(Name).GetPlainNameString();
	
		// Allow for partial matches of the name.
		auto MatchesPartOfName = [&Name, &PlainNameString](FName i) -> bool
			{
				return Name.Contains(i.ToString()) || (PlainNameString == i.GetPlainNameString());
			};

		// Solo active?
		if (Solos.Num() > 0)
		{
			if (Solos.ContainsByPredicate(MatchesPartOfName))
			{
				bOutIsSoloed = true;
				OutReason = FString::Printf(TEXT("Sound is soloed explicitly."));
			}
			else
			{
				// Something else is soloed (record the first item in the solo list for debug reason).
				bOutIsMuted = true;
				OutReason = FString::Printf(TEXT("Sound is muted due to [%s] being soloed"), *Solos[0].ToString() );
			}
		}
		// Are we explicitly muted?
		else if (Mutes.ContainsByPredicate(MatchesPartOfName))
		{
			bOutIsMuted = true;
			OutReason = FString::Printf(TEXT("Sound is explicitly muted"));
		}
	}

	void FAudioDebugger::DrawDebugInfo(const FSoundSource& SoundSource)
	{
	#if ENABLE_DRAW_DEBUG
		const FWaveInstance* WaveInstance = SoundSource.GetWaveInstance();
		if (!WaveInstance)
		{
			return;
		}

		const FActiveSound* ActiveSound = WaveInstance->ActiveSound;
		if (!ActiveSound)
		{
			return;
		}

		if (!SpatialSourceVisualizeEnabledCVar)
		{
			return;
		}

		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		if (DeviceManager && DeviceManager->IsVisualizeDebug3dEnabled())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.DrawSourceDebugInfo"), STAT_AudioDrawSourceDebugInfo, STATGROUP_TaskGraphTasks);
			
			const bool bSpatialized = SoundSource.GetNumChannels() == 2 && WaveInstance->GetUseSpatialization();
			if (bSpatialized)
			{
				const FRotator Rotator = ActiveSound->Transform.GetRotation().Rotator();

				TWeakObjectPtr<UWorld> WorldPtr = WaveInstance->ActiveSound->GetWeakWorld();
				FVector LeftChannelSourceLoc;
				FVector RightChannelSourceLoc;
				SoundSource.GetChannelLocations(LeftChannelSourceLoc, RightChannelSourceLoc);
				FAudioThread::RunCommandOnGameThread([LeftChannelSourceLoc, RightChannelSourceLoc, Rotator, WorldPtr]()
				{
					if (WorldPtr.IsValid())
					{
						UWorld* World = WorldPtr.Get();
						DrawDebugCrosshairs(World, LeftChannelSourceLoc, Rotator, 20.0f, FColor::Red, false, -1.0f, SDPG_Foreground);
						DrawDebugCrosshairs(World, RightChannelSourceLoc, Rotator, 20.0f, FColor::Green, false, -1.0f, SDPG_Foreground);
					}
				}, GET_STATID(STAT_AudioDrawSourceDebugInfo));
			}
		}
	#endif // ENABLE_DRAW_DEBUG
	}

	void FAudioDebugger::DrawDebugInfo(const FActiveSound& ActiveSound, const TArray<FWaveInstance*>& ThisSoundsWaveInstances, const float DeltaTime)
	{
	#if ENABLE_DRAW_DEBUG
		if (!ActiveSoundVisualizeModeCVar)
		{
			return;
		}

		// Only draw spatialized sounds
		const USoundBase* Sound = ActiveSound.GetSound();
		if (!Sound || !ActiveSound.bAllowSpatialization)
		{
			return;
		}

		const float PlaybackTime = ActiveSound.PlaybackTime;
		const float PlaybackTimeNonVirtualized = ActiveSound.PlaybackTimeNonVirtualized;
		const bool bOccluded = ActiveSound.bIsOccluded;

		// Sounds requiring culling can start and immediately stop repeatedly when subscribed
		// concurrency is flooded, so don't show the initial frame.
		if (FMath::IsNearlyZero(PlaybackTimeNonVirtualized))
		{
			return;
		}

		if (ActiveSoundVisualizeTypeCVar > 0)
		{
			if (ActiveSoundVisualizeTypeCVar == 1 && ActiveSound.GetAudioComponentID() == 0)
			{
				return;
			}

			if (ActiveSoundVisualizeTypeCVar == 2 && ActiveSound.GetAudioComponentID() > 0)
			{
				return;
			}
		}

		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		if (DeviceManager && DeviceManager->IsVisualizeDebug3dEnabled())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.DrawActiveSoundDebugInfo"), STAT_AudioDrawActiveSoundDebugInfo, STATGROUP_TaskGraphTasks);

			const FString Name = Sound->GetName();
			const FTransform CurTransform = ActiveSound.Transform;
			FColor TextColor = FColor::White;
			const float CurMaxDistance = ActiveSound.MaxDistance;
			float DisplayValue = 0.0f;
			float FilterValue = 0.0f;
			if (ActiveSoundVisualizeModeCVar == 1 || ActiveSoundVisualizeModeCVar == 2)
			{
				for (FWaveInstance* WaveInstance : ThisSoundsWaveInstances)
				{
					DisplayValue = FMath::Max(DisplayValue, WaveInstance->GetVolumeWithDistanceAndOcclusionAttenuation() * WaveInstance->GetDynamicVolume());
				}
			}
			else if (ActiveSoundVisualizeModeCVar == 3)
			{
				if (ActiveSound.AudioDevice)
				{
					DisplayValue = ActiveSound.AudioDevice->GetDistanceToNearestListener(ActiveSound.Transform.GetLocation()) / CurMaxDistance;
				}
			}
			else if (ActiveSoundVisualizeModeCVar == 4)
			{
				TextColor = ActiveSound.DebugColor;
			}
			else if (ActiveSoundVisualizeModeCVar == 5)
			{
				DisplayValue = ActiveSound.CurrentOcclusionVolumeAttenuation.GetValue();
				FilterValue = ActiveSound.CurrentOcclusionFilterFrequency.GetValue();
			}

			TWeakObjectPtr<UWorld> WorldPtr = ActiveSound.GetWeakWorld();
			FAudioThread::RunCommandOnGameThread([Name, TextColor, CurTransform, DisplayValue, WorldPtr, CurMaxDistance, PlaybackTime, PlaybackTimeNonVirtualized, bOccluded, FilterValue, DeltaTime]()
			{
				if (WorldPtr.IsValid())
				{
					static const float ColorRedHue = 0.0f;
					static const float ColorGreenHue = 85.0f;

					const FVector Location = CurTransform.GetLocation();
					UWorld* DebugWorld = WorldPtr.Get();
					DrawDebugSphere(DebugWorld, Location, 10.0f, 8, FColor::White, false, -1.0f, SDPG_Foreground);
					FColor Color = TextColor;

					FString Descriptor;
					if (ActiveSoundVisualizeModeCVar == 1 || ActiveSoundVisualizeModeCVar == 2)
					{
						const float DisplayDbVolume = Audio::ConvertToDecibels(DisplayValue);
						if (ActiveSoundVisualizeModeCVar == 1)
						{
							Descriptor = FString::Printf(TEXT(" (Vol: %.3f [Active: %.2fs, Playing: %.2fs])"), DisplayValue, PlaybackTime, PlaybackTimeNonVirtualized);
						}
						else
						{
							Descriptor = FString::Printf(TEXT(" (Vol: %.3f dB [Active: %.2fs, Playing: %.2fs])"), DisplayDbVolume, PlaybackTime, PlaybackTimeNonVirtualized);
						}
						static const float DbColorMinVol = -30.0f;
						const float DbVolume = FMath::Clamp(DisplayDbVolume, DbColorMinVol, 0.0f);
						const float Hue = FMath::Lerp(ColorRedHue, ColorGreenHue, (-1.0f * DbVolume / DbColorMinVol) + 1.0f);
						Color = FLinearColor::MakeFromHSV8(static_cast<uint8>(Hue), 255u, 255u).ToFColor(true);
					}
					else if (ActiveSoundVisualizeModeCVar == 3)
					{
						Descriptor = FString::Printf(TEXT(" (Dist: %.3f, Max: %.3f)"), DisplayValue * CurMaxDistance, CurMaxDistance);
						const float Hue = FMath::Lerp(ColorGreenHue, ColorRedHue, DisplayValue);
						Color = FLinearColor::MakeFromHSV8(static_cast<uint8>(FMath::Clamp(Hue, 0.0f, 255.f)), 255u, 255u).ToFColor(true);
					}
					else if (ActiveSoundVisualizeModeCVar == 5)
					{
						Descriptor = FString::Printf(TEXT(" (Occlusion Volume: %.3f, Occlusion Filter: %.3f)"), DisplayValue, FilterValue);
						if (bOccluded)
						{
							Color = FColor::Red;
						}
						else
						{
							Color = FColor::Green;
						}
					}

					const FString Description = FString::Printf(TEXT("%s%s"), *Name, *Descriptor);
					DrawDebugString(DebugWorld, Location + FVector(0, 0, 32), *Description, nullptr, Color, DeltaTime, false);
				}
			}, GET_STATID(STAT_AudioDrawActiveSoundDebugInfo));
		}
	#endif // ENABLE_DRAW_DEBUG
	}

	void FAudioDebugger::DrawDebugInfo(UWorld& World, const TArray<FListener>& Listeners)
	{
	#if ENABLE_DRAW_DEBUG
		if (!ActiveSoundVisualizeListenersCVar)
		{
			return;
		}

		check(IsInAudioThread());

		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		if (DeviceManager && DeviceManager->IsVisualizeDebug3dEnabled())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.DrawListenerDebugInfo"), STAT_AudioDrawListenerDebugInfo, STATGROUP_TaskGraphTasks);
			const TWeakObjectPtr<UWorld> WorldPtr = &World;
			for (const FListener& Listener : Listeners)
			{
				const FVector ListenerPosition = Listener.GetPosition(true);
				const FVector ListenerFront = Listener.GetFront();
				const FVector ListenerUp = Listener.GetUp();
				const FVector ListenerRight = Listener.GetRight();

				FAudioThread::RunCommandOnGameThread([WorldPtr, ListenerPosition, ListenerFront, ListenerUp, ListenerRight]()
				{
					if (WorldPtr.IsValid())
					{
						static float ArrowLength	 = 30.0f;
						static float ArrowHeadSize	 = 8.0f;
						static float Lifetime		 = -1.0f;
						static float Thickness		 = 0.9f;
						static uint8 DepthPriority	 = SDPG_Foreground;
						static bool bPersistentLines = false;

						DrawDebugDirectionalArrow(WorldPtr.Get(), ListenerPosition, ListenerPosition + (ArrowLength * ListenerFront), ArrowHeadSize, FColor::Red, bPersistentLines, Lifetime, DepthPriority, Thickness);
						DrawDebugDirectionalArrow(WorldPtr.Get(), ListenerPosition, ListenerPosition + (ArrowLength * ListenerUp), ArrowHeadSize, FColor::Blue, bPersistentLines, Lifetime, DepthPriority, Thickness);
						DrawDebugDirectionalArrow(WorldPtr.Get(), ListenerPosition, ListenerPosition + (ArrowLength * ListenerRight), ArrowHeadSize, FColor::Green, bPersistentLines, Lifetime, DepthPriority, Thickness);
						DrawDebugSphere(WorldPtr.Get(), ListenerPosition, 5.0f, 8, FColor::Magenta, bPersistentLines, Lifetime, DepthPriority);
					}
				}, GET_STATID(STAT_AudioDrawListenerDebugInfo));
			}
		}
	#endif // ENABLE_DRAW_DEBUG
	}

	void FAudioDebugger::DrawDebugInfo(const FAudioVirtualLoop& VirtualLoop)
	{
	#if ENABLE_DRAW_DEBUG
		if (!GEngine)
		{
			return;
		}

		if (!VirtualLoopsVisualizeEnabledCVar)
		{
			return;
		}

		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		if (DeviceManager && DeviceManager->IsVisualizeDebug3dEnabled())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.DrawVirtualLoopDebugInfo"), STAT_AudioDrawVirtualLoopDebugInfo, STATGROUP_TaskGraphTasks);

			const FActiveSound& ActiveSound = VirtualLoop.GetActiveSound();
			USoundBase* Sound = ActiveSound.GetSound();
			check(Sound);

			const FTransform Transform = ActiveSound.Transform;
			const TWeakObjectPtr<UWorld> World = ActiveSound.GetWeakWorld();
			const FString Name = Sound->GetName();
			const float DrawInterval = VirtualLoop.GetUpdateInterval();
			const float TimeVirtualized = VirtualLoop.GetTimeVirtualized();
			FAudioThread::RunCommandOnGameThread([World, Transform, Name, DrawInterval, TimeVirtualized]()
			{
				if (World.IsValid())
				{
					const FString Description = FString::Printf(TEXT("%s [Virt: %.2fs]"), *Name, TimeVirtualized);
					FVector Location = Transform.GetLocation();
					FRotator Rotation = Transform.GetRotation().Rotator();
					DrawDebugCrosshairs(World.Get(), Location, Rotation, 20.0f, FColor::Blue, false, DrawInterval, SDPG_Foreground);
					DrawDebugString(World.Get(), Location + FVector(0, 0, 32), *Description, nullptr, FColor::Blue, DrawInterval, false);
				}
			}, GET_STATID(STAT_AudioDrawVirtualLoopDebugInfo));
		}
	#endif // ENABLE_DRAW_DEBUG
	}

	int32 FAudioDebugger::DrawDebugStats(UWorld& World, FViewport* OutViewport, FCanvas* InCanvas, int32 InY)
	{
		check(IsInGameThread());

		if (OutViewport)
		{
			return DrawDebugStatsInternal(World, *OutViewport, InCanvas, InY);
		}

		if (UGameViewportClient* GameViewport = World.GetGameViewport())
		{
			if (GameViewport->Viewport)
			{
				return DrawDebugStatsInternal(World, *GameViewport->Viewport, InCanvas, InY);
			}

			return InY;
		}

#if WITH_EDITOR
		if (GEditor)
		{
			// Level editor viewport may not be a game viewport, but should still display debug stats, so check if this case is covered and display accordingly
			const TArray<FLevelEditorViewportClient*>& LevelViewportClients = GEditor->GetLevelViewportClients();
			for (FLevelEditorViewportClient* ViewportClient : LevelViewportClients)
			{
				if (ViewportClient && ViewportClient->Viewport)
				{
					UWorld* EditorWorld = ViewportClient->GetWorld();
					if (EditorWorld && EditorWorld == &World)
					{
						const FText SystemDisplayName = NSLOCTEXT("AudioDebugger", "AudioDebugger_DrawDebugData", "Audio Debug Data");
						if (DrawDebugStatsEnabled())
						{
							ViewportClient->AddRealtimeOverride(true, SystemDisplayName);
						}

						ViewportClient->RemoveRealtimeOverride(SystemDisplayName);

						return DrawDebugStatsInternal(World, *ViewportClient->Viewport, InCanvas, InY);
					}
				}
			}
		}
#endif // WITH_EDITOR

		return InY;
	}

	bool FAudioDebugger::DrawDebugStatsEnabled()
	{
		return bDebugWavesForAllViewsEnabled
			|| bDebugCuesForAllViewsEnabled
			|| bDebugSoundsForAllViewsEnabled
			|| bDebugMixesForAllViewsEnabled
			|| bDebugReverbForAllViewsEnabled
			|| bDebugModulationForAllViewsEnabled
			|| bDebugStreamingForAllViewsEnabled;
	}

	int32 FAudioDebugger::DrawDebugStatsInternal(UWorld& World, FViewport& Viewport, FCanvas* InCanvas, int32 InY)
	{
		FCanvas* Canvas = InCanvas ? InCanvas : Viewport.GetDebugCanvas();
		if (!Canvas)
		{
			return InY;
		}

		int32 X = SoundDebugDisplayCornerXCVar;
		int32 Y = SoundDebugDisplayCornerYCVar < 0 ? InY : SoundDebugDisplayCornerYCVar;

		Y = RenderStatMixes(&World, &Viewport, Canvas, X, Y);
		Y = RenderStatModulators(&World, &Viewport, Canvas, X, Y, nullptr, nullptr);
		Y = RenderStatReverb(&World, &Viewport, Canvas, X, Y);
		Y = RenderStatSounds(&World, &Viewport, Canvas, X, Y);
		Y = RenderStatCues(&World, &Viewport, Canvas, X, Y);
		Y = RenderStatWaves(&World, &Viewport, Canvas, X, Y);
		Y = RenderStatStreaming(&World, &Viewport, Canvas, X, Y, nullptr, nullptr);

		return Y;
	}

	void FAudioDebugger::DumpActiveSounds() const
	{
		if (!GEngine)
		{
			return;
		}

		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.DumpActiveSounds"), STAT_AudioDumpActiveSounds, STATGROUP_TaskGraphTasks);
			FAudioThread::RunCommandOnAudioThread([this]()
			{
				DumpActiveSounds();
			}, GET_STATID(STAT_AudioDumpActiveSounds));
			return;
		}

		FAudioDeviceHandle AudioDevice = GEngine->GetAudioDeviceManager()->GetActiveAudioDevice();
		if (!AudioDevice)
		{
			return;
		}

		const TArray<FActiveSound*>& ActiveSounds = AudioDevice->GetActiveSounds();
		UE_LOGF(LogAudio, Display, "Active Sound Count: %d", ActiveSounds.Num());
		UE_LOGF(LogAudio, Display, "------------------------");

		for (const FActiveSound* ActiveSound : ActiveSounds)
		{
			if (ActiveSound)
			{
				UE_LOGF(LogAudio, Display, "%ls (%.3g) - %ls", *ActiveSound->GetSound()->GetName(), ActiveSound->GetSound()->GetDuration(), *ActiveSound->GetAudioComponentName());

				for (const TPair<UPTRINT, FWaveInstance*>& WaveInstancePair : ActiveSound->GetWaveInstances())
				{
					const FWaveInstance* WaveInstance = WaveInstancePair.Value;
					UE_LOG(LogAudio, Display, TEXT("   %s (%.3g) (%" SIZE_T_FMT ") - %.3g"),
						*WaveInstance->GetName(), WaveInstance->WaveData->GetDuration(),
						WaveInstance->WaveData->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal),
						WaveInstance->GetVolumeWithDistanceAndOcclusionAttenuation() * WaveInstance->GetDynamicVolume());
				}
			}
		}
	}

	void FAudioDebugger::ResolveDesiredStats(FViewportClient* ViewportClient)
	{
		if (!ViewportClient)
		{
			return;
		}

		FAudioDevice* AudioDevice = GetWorldAudio(ViewportClient->GetWorld());
		if (!AudioDevice)
		{
			return;
		}

		TSet<FName> SetStatFlags;

		if (ViewportClient->IsStatEnabled(TEXT("SoundCues")))
		{
			SetStatFlags.Add(DebugStatNames::SoundCues);
		}

		if (ViewportClient->IsStatEnabled(TEXT("SoundWaves")))
		{
			SetStatFlags.Add(DebugStatNames::SoundWaves);
		}

		if (ViewportClient->IsStatEnabled(TEXT("SoundMixes")))
		{
			SetStatFlags.Add(DebugStatNames::SoundMixes);
		}

		if (ViewportClient->IsStatEnabled(TEXT("Sounds")))
		{
			FAudioStats& Stats = AudioDeviceStats.FindOrAdd(AudioDevice->DeviceID);
			SetStatFlags.Add(DebugStatNames::Sounds);

			if (Stats.DisplayFlags & static_cast<uint8>(FAudioStats::EDisplayFlags::Debug))
			{
				SetStatFlags.Add(DebugStatNames::DebugSounds);
			}

			if (Stats.DisplayFlags & static_cast<uint8>(FAudioStats::EDisplayFlags::Long_Names))
			{
				SetStatFlags.Add(DebugStatNames::LongSoundNames);
			}
		}

		if (ViewportClient->IsStatEnabled(TEXT("Streaming")))
		{
			SetStatFlags.Add(DebugStatNames::AudioStreaming);
		}

		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ResolveDesiredStats"), STAT_AudioResolveDesiredStats, STATGROUP_TaskGraphTasks);

		const uint32 DeviceID = AudioDevice->DeviceID;
		if (IsInAudioThread())
		{
			FAudioStats_AudioThread& Stats = AudioDeviceStats_AudioThread.FindOrAdd(DeviceID);
			Stats.RequestedStats = SetStatFlags;
		}
		else
		{
			FAudioThread::RunCommandOnAudioThread([SetStatFlags, DeviceID]()
			{
				FAudioStats_AudioThread& Stats = AudioDeviceStats_AudioThread.FindOrAdd(DeviceID);
				Stats.RequestedStats = SetStatFlags;
			}, GET_STATID(STAT_AudioResolveDesiredStats));
		}
	}

	int32 FAudioDebugger::RenderStatCues(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y)
	{
		FAudioDevice* AudioDevice = nullptr;
		if (!DebugShouldRenderStat(World, Canvas, bDebugCuesForAllViewsEnabled, DebugStatNames::SoundCues, &AudioDevice))
		{
			return Y;
		}

		UFont* StatsFont = GetStatsFont();
		const int32 FontSpacing = 2;
		const int32 FontHeight = StatsFont->GetMaxCharHeight() + FontSpacing;

		Canvas->DrawShadowedString(X, Y, TEXT("Active Sound Cues:"), StatsFont, HeaderColor);
		Y += FontHeight;

		int32 ActiveSoundCount = 0;

		FAudioStats& AudioStats = AudioDeviceStats.FindOrAdd(AudioDevice->DeviceID);
		if (!SoundCueDebugMinimalCVar)
		{
			for (const FAudioStats::FStatSoundInfo& StatSoundInfo : AudioStats.StatSoundInfos)
			{
				for (const FAudioStats::FStatWaveInstanceInfo& WaveInstanceInfo : StatSoundInfo.WaveInstanceInfos)
				{
					if (WaveInstanceInfo.Volume >= MinDisplayVolume)
					{
						FColor Color = FColor::White;
						FString MuteSoloReason;

						if (FSoundSource::FDebugInfo* DebugInfo = WaveInstanceInfo.DebugInfo.Get())
						{
							// Color code same as icons. Red (mute), Yellow (solo), White (normal).
							FScopeLock Lock(&DebugInfo->CS);
							Color = DebugInfo->bIsMuted ? FColor::Red : DebugInfo->bIsSoloed ? FColor::Yellow : FColor::White;
							MuteSoloReason = !DebugInfo->MuteSoloReason.IsEmpty() ? FString::Printf(TEXT(" - %s"), *DebugInfo->MuteSoloReason) : TEXT("");
						}

						const FString TheString = FString::Printf(TEXT("%4i. %6.2f %s %s %s"), ActiveSoundCount++, WaveInstanceInfo.Volume, *StatSoundInfo.SoundName, *StatSoundInfo.SoundClassName.ToString(), *MuteSoloReason);
						Canvas->DrawShadowedString(X, Y, *TheString, StatsFont, Color);
						Y += FontHeight;
						break;
					}
				}
			}
		}
		else
		{
			FString SoundPath, SoundName;
			static int32 PrevNameLength = 0;
			static int32 PrevClassLength = 0;

			const int32 TabSpacing = FMath::Clamp(SoundCueDebugTabSpacingCVar, 1, SoundCueDebugTabSpacingCVar);
			const int32 CharSpacing = FMath::Clamp(SoundCueDebugCharSpacingCVar, 1, SoundCueDebugCharSpacingCVar);
			const int32 NumberSpacing = 6 * CharSpacing;	// 6 character len for 2 decimal float + 2 spaces 'X.XX  '

			// Tab out name and class length and reset previous length counters
			int32 TabbedName = (PrevNameLength / TabSpacing + 1) * TabSpacing;
			int32 TabbedClass = (PrevClassLength / TabSpacing + 1) * TabSpacing;
			PrevNameLength = PrevClassLength = 0;

			for (const FAudioStats::FStatSoundInfo& StatSoundInfo : AudioStats.StatSoundInfos)
			{
				for (const FAudioStats::FStatWaveInstanceInfo& WaveInstanceInfo : StatSoundInfo.WaveInstanceInfos)
				{
					if (WaveInstanceInfo.Volume >= MinDisplayVolume)
					{
						FColor Color = FColor::White;
						FString MuteSoloReason;
						bool bMutedOrSoloed = false;

						if (!StatSoundInfo.SoundPath.Split(TEXT("."), &SoundPath, &SoundName))
						{
							if (!StatSoundInfo.SoundPath.Split(SUBOBJECT_DELIMITER, &SoundPath, &SoundName))
							{
								SoundPath = StatSoundInfo.SoundPath;
							}
						}
						SoundName = StatSoundInfo.SoundName;

						if (SoundName.Len() > PrevNameLength)
						{
							PrevNameLength = SoundName.Len();
						}

						if ((int32)StatSoundInfo.SoundClassName.GetStringLength() > PrevClassLength)
						{
							PrevClassLength = StatSoundInfo.SoundClassName.GetStringLength();
						}

						if (FSoundSource::FDebugInfo* DebugInfo = WaveInstanceInfo.DebugInfo.Get())
						{
							// Color code same as icons. Red (mute), Yellow (solo), White (normal).
							FScopeLock Lock(&DebugInfo->CS);
							Color = DebugInfo->bIsMuted ? FColor::Red : DebugInfo->bIsSoloed ? FColor::Yellow : FColor::White;
							bMutedOrSoloed = DebugInfo->bIsMuted || DebugInfo->bIsSoloed;
							MuteSoloReason = !DebugInfo->MuteSoloReason.IsEmpty() ? FString::Printf(TEXT(" - %s"), *DebugInfo->MuteSoloReason) : TEXT("");
						}

						const int32 SoundNameIndex = ActiveSoundCount++;
						const FString LeadingNumber = FString::Printf(TEXT("%4i. "), SoundNameIndex);
						const FString Volume = FString::Printf(TEXT("%6.3f "), WaveInstanceInfo.Volume);
						const FString PathAndMuting = FString::Printf(TEXT("Path: %s %s"), *SoundPath, *MuteSoloReason);

						int32 CurrentX = X;
						Canvas->DrawShadowedString(CurrentX, Y, *LeadingNumber, StatsFont, Color);
						CurrentX += NumberSpacing;
						Canvas->DrawShadowedString(CurrentX, Y, *Volume, StatsFont, bMutedOrSoloed ? Color : FColor::Orange);
						CurrentX += NumberSpacing;

						if (SoundCueDebugShowDistanceCVar)
						{
							const FString DistanceText = FString::Printf(TEXT("%6.2f "), StatSoundInfo.Distance);
							Canvas->DrawShadowedString(CurrentX, Y, *DistanceText, StatsFont, bMutedOrSoloed ? Color : FColor::White);
							CurrentX += (NumberSpacing * 2);
						}

						Canvas->DrawShadowedString(CurrentX, Y, *SoundName, StatsFont, bMutedOrSoloed ? Color : FColor(0, 255, 255));
						CurrentX += (TabbedName * CharSpacing);
						Canvas->DrawShadowedString(CurrentX, Y, *StatSoundInfo.SoundClassName.ToString(), StatsFont, bMutedOrSoloed ? Color : FColor::Yellow);

						if (SoundCueDebugShowPathCVar)
						{
							CurrentX += (TabbedClass * CharSpacing);
							Canvas->DrawShadowedString(CurrentX, Y, *PathAndMuting, StatsFont, Color);
						}

						Y += FontHeight;
						break;
					}
				}
			}
		}

		Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("Total: %i"), ActiveSoundCount), GetStatsFont(), GetBodyColor());
		Y += FontHeight;

		return Y;
	}

	int32 FAudioDebugger::RenderStatMixes(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y)
	{
		FAudioDevice* AudioDevice = nullptr;
		if (!DebugShouldRenderStat(World, Canvas, bDebugMixesForAllViewsEnabled, DebugStatNames::SoundMixes, &AudioDevice))
		{
			return Y;
		}

		const int32 FontHeight = GetStatsFont()->GetMaxCharHeight() + 2;
		FAudioStats& AudioStats = AudioDeviceStats.FindOrAdd(AudioDevice->DeviceID);
		Canvas->DrawShadowedString(X, Y, TEXT("Active Sound Mixes:"), GetStatsFont(), HeaderColor);
		Y += FontHeight;

		bool bDisplayedSoundMixes = false;

		if (AudioStats.StatSoundMixes.Num() > 0)
		{
			bDisplayedSoundMixes = true;

			for (const FAudioStats::FStatSoundMix& StatSoundMix : AudioStats.StatSoundMixes)
			{
				const FString TheString = FString::Printf(TEXT("%s - Fade Proportion: %1.2f - Total Ref Count: %i"), *StatSoundMix.MixName, StatSoundMix.InterpValue, StatSoundMix.RefCount);

				const FColor& TextColor = (StatSoundMix.bIsCurrentEQ ? FColor::Yellow : GetBodyColor());

				Canvas->DrawShadowedString(X + DebuggerTabWidth, Y, *TheString, GetStatsFont(), TextColor);
				Y += FontHeight;
			}
		}

		if (!bDisplayedSoundMixes)
		{
			Canvas->DrawShadowedString(X + DebuggerTabWidth, Y, TEXT("None"), GetStatsFont(), GetBodyColor());
			Y += FontHeight;
		}

		Y += FontHeight;
		return Y;
	}

	int32 FAudioDebugger::RenderStatModulators(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		FAudioDevice* AudioDevice = nullptr;
		if (!DebugShouldRenderStat(World, Canvas, bDebugModulationForAllViewsEnabled, DebugStatNames::SoundModulation, &AudioDevice))
		{
			return Y;
		}

		const int32 FontHeight = GetStatsFont()->GetMaxCharHeight() + 2;
		Canvas->DrawShadowedString(X, Y, TEXT("Active Sound Modulation:"), GetStatsFont(), HeaderColor);
		Y += FontHeight;

		bool bDisplayedSoundModulationInfo = false;
		if (IAudioModulationManager* Modulation = AudioDevice->ModulationInterface.Get())
		{
			const int32 YInit = Y;
			Y = Modulation->OnRenderStat(Viewport, Canvas, X, Y, *GetStatsFont(), ViewLocation, ViewRotation);
			bDisplayedSoundModulationInfo = Y != YInit;
		}

		if (!bDisplayedSoundModulationInfo)
		{
			Canvas->DrawShadowedString(X + DebuggerTabWidth, Y, TEXT("None"), GetStatsFont(), GetBodyColor());
			Y += FontHeight;
		}

		Y += FontHeight;
		return Y;
	}

	int32 FAudioDebugger::RenderStatReverb(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y)
	{
		FAudioDevice* AudioDevice = nullptr;
		if (!DebugShouldRenderStat(World, Canvas, bDebugReverbForAllViewsEnabled, DebugStatNames::SoundReverb, &AudioDevice))
		{
			return Y;
		}

		const int32 Height = static_cast<int32>(GetStatsFont()->GetMaxCharHeight() + 2);

		FString TheString;
		const FLinearColor LinearBodyColor = FLinearColor(GetBodyColor());
		if (UReverbEffect* ReverbEffect = AudioDevice->GetCurrentReverbEffect())
		{
			TheString = FString::Printf(TEXT("Active Reverb Effect: %s"), *ReverbEffect->GetName());
			Canvas->DrawShadowedString(X, Y, *TheString, GetStatsFont(), HeaderColor);
			Y += Height;

			AAudioVolume* CurrentAudioVolume = nullptr;
			const int32 ProxyCount = AudioDevice->ListenerProxies.Num();
			for (int i = 0; i < ProxyCount; ++i)
			{
				const FTransform& Transform = AudioDevice->ListenerProxies[i].Transform;
				AAudioVolume* PlayerAudioVolume = World->GetAudioSettings(Transform.GetLocation(), nullptr, nullptr);
				if (PlayerAudioVolume && ((CurrentAudioVolume == nullptr) || (PlayerAudioVolume->GetPriority() > CurrentAudioVolume->GetPriority())))
				{
					CurrentAudioVolume = PlayerAudioVolume;
				}
			}
			if (CurrentAudioVolume && CurrentAudioVolume->GetReverbSettings().ReverbEffect)
			{
				TheString = FString::Printf(TEXT("  Audio Volume Reverb Effect: %s (Priority: %g Volume Name: %s)"), *CurrentAudioVolume->GetReverbSettings().ReverbEffect->GetName(), CurrentAudioVolume->GetPriority(), *CurrentAudioVolume->GetName());
			}
			else
			{
				TheString = TEXT("  Audio Volume Reverb Effect: None");
			}
			Canvas->DrawShadowedString(X, Y, *TheString, GetStatsFont(), LinearBodyColor);
			Y += Height;

			const TMap<FName, FActivatedReverb>& ActivatedReverbs = AudioDevice->GetActiveReverb();
			if (ActivatedReverbs.Num() == 0)
			{
				TheString = TEXT("  Activated Reverb: None");
				Canvas->DrawShadowedString(X, Y, *TheString, GetStatsFont(), LinearBodyColor);
				Y += Height;
			}
			else if (ActivatedReverbs.Num() == 1)
			{
				auto It = ActivatedReverbs.CreateConstIterator();
				const FActivatedReverb& ActiveReverb = It.Value();
				if (ActiveReverb.ReverbSettings.ReverbEffect)
				{
					TheString = FString::Printf(TEXT("  Activated Reverb Effect: %s (Priority: %g Tag: '%s')"), *ActiveReverb.ReverbSettings.ReverbEffect->GetName(), ActiveReverb.Priority, *It.Key().ToString());
					Canvas->DrawShadowedString(X, Y, *TheString, GetStatsFont(), LinearBodyColor);
					Y += Height;
				}
			}
			else
			{
				Canvas->DrawShadowedString(X, Y, TEXT("  Activated Reverb Effects:"), GetStatsFont(), LinearBodyColor);
				Y += Height;
				TMap<int32, FString> PrioritySortedActivatedReverbs;
				for (auto It = ActivatedReverbs.CreateConstIterator(); It; ++It)
				{
					const FActivatedReverb& ActiveReverb = It.Value();
					if (ActiveReverb.ReverbSettings.ReverbEffect)
					{
						TheString = FString::Printf(TEXT("    %s (Priority: %g Tag: '%s')"), *ActiveReverb.ReverbSettings.ReverbEffect->GetName(), ActiveReverb.Priority, *It.Key().ToString());
						PrioritySortedActivatedReverbs.Add(ActiveReverb.Priority, TheString);
					}
				}
				for (auto It = PrioritySortedActivatedReverbs.CreateConstIterator(); It; ++It)
				{
					Canvas->DrawShadowedString(X, Y, *It.Value(), GetStatsFont(), LinearBodyColor);
					Y += Height;
				}
			}
		}
		else
		{
			TheString = TEXT("Active Reverb Effect: None");
			Canvas->DrawShadowedString(X, Y, *TheString, GetStatsFont(), HeaderColor);
			Y += Height;
		}

		Y += Height;
		return Y;
	}

	int32 FAudioDebugger::RenderStatSounds(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y)
	{
		FAudioDevice* AudioDevice = nullptr;
		if (!DebugShouldRenderStat(World, Canvas, bDebugSoundsForAllViewsEnabled, DebugStatNames::Sounds, &AudioDevice))
		{
			return Y;
		}

		const int32 FontHeight = GetStatsFont()->GetMaxCharHeight() + 2;

		FAudioStats& AudioStats = AudioDeviceStats.FindOrAdd(AudioDevice->DeviceID);
		const uint8 bDebug = AudioStats.DisplayFlags & static_cast<uint8>(FAudioStats::EDisplayFlags::Debug);
		UpdateDisplaySort(AudioStats);

		// Sort the list.
		FString SortingName;
		switch (AudioStats.DisplaySort)
		{
			case FAudioStats::EDisplaySort::Class:
		{
				AudioStats.StatSoundInfos.Sort([](const FAudioStats::FStatSoundInfo& A, const FAudioStats::FStatSoundInfo& B) { return A.SoundClassName.LexicalLess(B.SoundClassName); });
				SortingName = TEXT("Class");
				break;
		}
			case FAudioStats::EDisplaySort::Distance:
		{
			AudioStats.StatSoundInfos.Sort([](const FAudioStats::FStatSoundInfo& A, const FAudioStats::FStatSoundInfo& B) { return A.Distance < B.Distance; });
				SortingName = TEXT("Distance");
				break;
		}
			case FAudioStats::EDisplaySort::PlaybackTime:
		{
				AudioStats.StatSoundInfos.Sort([](const FAudioStats::FStatSoundInfo& A, const FAudioStats::FStatSoundInfo& B) { return A.PlaybackTime > B.PlaybackTime; });
				SortingName = TEXT("Time");
				break;
			}
			case FAudioStats::EDisplaySort::Priority:
			{
				AudioStats.StatSoundInfos.Sort([](const FAudioStats::FStatSoundInfo& A, const FAudioStats::FStatSoundInfo& B) { return A.Priority > B.Priority; });
				SortingName = TEXT("Priority");
				break;
		}
			case FAudioStats::EDisplaySort::Waves:
		{
			AudioStats.StatSoundInfos.Sort([](const FAudioStats::FStatSoundInfo& A, const FAudioStats::FStatSoundInfo& B) { return A.WaveInstanceInfos.Num() > B.WaveInstanceInfos.Num(); });
				SortingName = TEXT("Waves");
				break;
			}
			case FAudioStats::EDisplaySort::Volume:
			{
				AudioStats.StatSoundInfos.Sort([](const FAudioStats::FStatSoundInfo& A, const FAudioStats::FStatSoundInfo& B) { return A.Volume > B.Volume; });
				SortingName = TEXT("Volume");
				break;
			}
			case FAudioStats::EDisplaySort::Name:
			default:
			{
				AudioStats.StatSoundInfos.Sort([](const FAudioStats::FStatSoundInfo& A, const FAudioStats::FStatSoundInfo& B)
				{
					if (AudioDebugSoundShowPathCVar)
					{
						return A.SoundPath < B.SoundPath;
					}
					return A.SoundName < B.SoundName;
				});
				SortingName = TEXT("Name");
				break;
			}
		}

		Canvas->DrawShadowedString(X, Y, TEXT("Active Sounds:"), GetStatsFont(), HeaderColor);
		Y += FontHeight;

		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		const FString InfoText = FString::Printf(TEXT(" Sorting By: %s, 3D Visualization: %s"), *SortingName, 
			DeviceManager->IsVisualizeDebug3dEnabled() || bAttenuationVisualizeEnabledCVar ? TEXT("Enabled") : TEXT("Disabled"));
		Canvas->DrawShadowedString(X, Y, *InfoText, GetStatsFont(), FColor(128, 255, 128));
		Y += FontHeight;

		static int32 FieldLength = 40;
		static int32 NameHeaderPad = static_cast<int32>(FieldLength * 1.25f);
		if (AudioStats.DisplaySort == FAudioStats::EDisplaySort::Name || AudioStats.DisplaySort == FAudioStats::EDisplaySort::Distance)
		{
			const FString FieldName = FString("Distance").RightPad(NameHeaderPad).Left(NameHeaderPad);
			Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("Index %s Name"), *FieldName), GetStatsFont(), GetBodyColor());
		}
		else
		{
			const FString FieldName = SortingName.RightPad(FieldLength).Left(FieldLength);
			Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("Index %s Name"), *FieldName), GetStatsFont(), GetBodyColor());
		}
		Y += FontHeight;

		int32 TotalSoundWavesNum = 0;
		const FColor BodyColor = GetStatSoundBodyColor();
		for (int32 SoundIndex = 0; SoundIndex < AudioStats.StatSoundInfos.Num(); ++SoundIndex)
		{
			const FAudioStats::FStatSoundInfo& StatSoundInfo = AudioStats.StatSoundInfos[SoundIndex];
			const int32 WaveInstancesNum = StatSoundInfo.WaveInstanceInfos.Num();
			if (WaveInstancesNum == 0)
			{
				continue;
			}
			TotalSoundWavesNum += WaveInstancesNum;

			if (SoundIndex >= AudioDebugSoundMaxNumDisplayedCVar)
			{
				continue;
			}

			bool bDisplayWaves = false;
			const FString DisplayName = AudioDebugSoundShowPathCVar ? StatSoundInfo.SoundPath : StatSoundInfo.SoundName;
			FString DebugValue;
			switch (AudioStats.DisplaySort)
			{
				case FAudioStats::EDisplaySort::Class:
				{
					DebugValue = *StatSoundInfo.SoundClassName.ToString();
					break;
				}
				case FAudioStats::EDisplaySort::PlaybackTime:
				{
					if (FMath::IsNearlyEqual(StatSoundInfo.PlaybackTime, StatSoundInfo.PlaybackTimeNonVirtualized))
					{
						DebugValue = FString::Printf(TEXT("%08.2f"), StatSoundInfo.PlaybackTime);
					}
					else
					{
						DebugValue = FString::Printf(TEXT("%08.2f (%08.2f Non-Virt)"), StatSoundInfo.PlaybackTime, StatSoundInfo.PlaybackTimeNonVirtualized);
					}
					break;
				}
				case FAudioStats::EDisplaySort::Priority:
				{
					if (FMath::IsNearlyEqual(StatSoundInfo.Priority, TNumericLimits<float>::Max()))
					{
						DebugValue += TEXT("Always  ");
					}
					else
					{
						DebugValue = FString::Printf(TEXT("%06.2f"), StatSoundInfo.Priority);
					}
					break;
				}
				case FAudioStats::EDisplaySort::Volume:
				{
					DebugValue = FString::Printf(TEXT("%01.2f (%04.2f dB)"), StatSoundInfo.Volume, Audio::ConvertToDecibels(StatSoundInfo.Volume));
					break;
				}
				case FAudioStats::EDisplaySort::Waves:
			{
					DebugValue = FString::Printf(TEXT("%03u"), StatSoundInfo.WaveInstanceInfos.Num());
					bDisplayWaves = true;
					break;
				}
				case FAudioStats::EDisplaySort::Name:
				case FAudioStats::EDisplaySort::Distance:
				default:
				{
					DebugValue = FString::Printf(TEXT("%08.2f"), StatSoundInfo.Distance);
					break;
				}
				}

			const FString DebugStr = FString::Printf(TEXT("  %03i    %s %s"), SoundIndex, *DebugValue.RightPad(FieldLength - 4).Left(FieldLength - 4), *DisplayName);
			Canvas->DrawShadowedString(X, Y, *DebugStr, GetStatsFont(), BodyColor);
			Y += FontHeight;

			if (bDisplayWaves)
			{
				for (int32 WaveIndex = 0; WaveIndex < WaveInstancesNum; WaveIndex++)
				{
					const FString WaveStr = *FString::Printf(TEXT("    %02i    %s"), WaveIndex, *StatSoundInfo.WaveInstanceInfos[WaveIndex].Description);
					Canvas->DrawShadowedString(X, Y, *WaveStr, GetStatsFont(), FColor(205, 205, 205));
					Y += FontHeight;
				}
			}
		}

		Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("Audio Device ID: %u, Max Sounds Displayed: %i"), AudioDevice->DeviceID, AudioDebugSoundMaxNumDisplayedCVar), GetStatsFont(), HeaderColor);
		Y += FontHeight;

		Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("Total Sounds: %i, Sound Waves: %i"), AudioStats.StatSoundInfos.Num(), TotalSoundWavesNum), GetStatsFont(), HeaderColor);
		Y += FontHeight;

		for (int32 i = 0; i < AudioStats.ListenerTransforms.Num(); ++i)
		{
			FString LocStr = AudioStats.ListenerTransforms[i].GetLocation().ToString();
			Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("Listener '%d' Position: %s"), i, *LocStr), GetStatsFont(), HeaderColor);
			Y += FontHeight;
		}

		Y += FontHeight;

		// Draw attenuation shape if enabled.
		for (const FAudioStats::FStatSoundInfo& StatSoundInfo : AudioStats.StatSoundInfos)
		{
			const FTransform& SoundTransform = StatSoundInfo.Transform;
			const int32 WaveInstancesNum = StatSoundInfo.WaveInstanceInfos.Num();

			if (StatSoundInfo.Distance > 100.0f && WaveInstancesNum > 0)
			{
				float SphereRadius = 0.f;
				float SphereInnerRadius = 0.f;

				if (StatSoundInfo.ShapeDetailsMap.Num() > 0)
				{
					const FString DebugName = AudioDebugSoundShowPathCVar ? StatSoundInfo.SoundPath : StatSoundInfo.SoundName;
					DrawDebugString(World, SoundTransform.GetTranslation(), *DebugName, nullptr, BodyColor, 0.01f);

					for (auto ShapeDetailsIt = StatSoundInfo.ShapeDetailsMap.CreateConstIterator(); ShapeDetailsIt; ++ShapeDetailsIt)
					{
						const FBaseAttenuationSettings::AttenuationShapeDetails& ShapeDetails = ShapeDetailsIt.Value();
						switch (ShapeDetailsIt.Key())
						{
						case EAttenuationShape::Sphere:
							if (ShapeDetails.Falloff > 0.f)
							{
								DrawDebugSphere(World, SoundTransform.GetTranslation(), ShapeDetails.Extents.X + ShapeDetails.Falloff, 10, FColor(155, 155, 255));
								DrawDebugSphere(World, SoundTransform.GetTranslation(), ShapeDetails.Extents.X, 10, FColor(55, 55, 255));
							}
							else
							{
								DrawDebugSphere(World, SoundTransform.GetTranslation(), ShapeDetails.Extents.X, 10, FColor(155, 155, 255));
							}
							break;

						case EAttenuationShape::Box:
							if (ShapeDetails.Falloff > 0.f)
							{
								DrawDebugBox(World, SoundTransform.GetTranslation(), ShapeDetails.Extents + FVector(ShapeDetails.Falloff), SoundTransform.GetRotation(), FColor(155, 155, 255));
								DrawDebugBox(World, SoundTransform.GetTranslation(), ShapeDetails.Extents, SoundTransform.GetRotation(), FColor(55, 55, 255));
							}
							else
							{
								DrawDebugBox(World, SoundTransform.GetTranslation(), ShapeDetails.Extents, SoundTransform.GetRotation(), FColor(155, 155, 255));
							}
							break;

						case EAttenuationShape::Capsule:

							if (ShapeDetails.Falloff > 0.f)
							{
								DrawDebugCapsule(World, SoundTransform.GetTranslation(), ShapeDetails.Extents.X + ShapeDetails.Falloff, ShapeDetails.Extents.Y + ShapeDetails.Falloff, SoundTransform.GetRotation(), FColor(155, 155, 255));
								DrawDebugCapsule(World, SoundTransform.GetTranslation(), ShapeDetails.Extents.X, ShapeDetails.Extents.Y, SoundTransform.GetRotation(), FColor(55, 55, 255));
							}
							else
							{
								DrawDebugCapsule(World, SoundTransform.GetTranslation(), ShapeDetails.Extents.X, ShapeDetails.Extents.Y, SoundTransform.GetRotation(), FColor(155, 155, 255));
							}
							break;

						case EAttenuationShape::Cone:
						{
							const FVector Origin = SoundTransform.GetTranslation() - (SoundTransform.GetUnitAxis(EAxis::X) * ShapeDetails.ConeOffset);

							if (ShapeDetails.Falloff > 0.f || ShapeDetails.Extents.Z > 0.f)
							{
								const float OuterAngle = FMath::DegreesToRadians(ShapeDetails.Extents.Y + ShapeDetails.Extents.Z);
								const float InnerAngle = FMath::DegreesToRadians(ShapeDetails.Extents.Y);
								DrawDebugCone(World, Origin, SoundTransform.GetUnitAxis(EAxis::X), ShapeDetails.Extents.X + ShapeDetails.Falloff + ShapeDetails.ConeOffset, OuterAngle, OuterAngle, 10, FColor(155, 155, 255));
								DrawDebugCone(World, Origin, SoundTransform.GetUnitAxis(EAxis::X), ShapeDetails.Extents.X + ShapeDetails.ConeOffset, InnerAngle, InnerAngle, 10, FColor(55, 55, 255));
							}
							else
							{
								const float Angle = FMath::DegreesToRadians(ShapeDetails.Extents.Y);
								DrawDebugCone(World, Origin, SoundTransform.GetUnitAxis(EAxis::X), ShapeDetails.Extents.X + ShapeDetails.ConeOffset, Angle, Angle, 10, FColor(155, 155, 255));
							}

							if (!FMath::IsNearlyZero(ShapeDetails.ConeSphereRadius, UE_KINDA_SMALL_NUMBER))
							{
								if (ShapeDetails.ConeSphereFalloff > 0.f)
								{

									DrawDebugSphere(World, Origin, ShapeDetails.ConeSphereRadius + ShapeDetails.ConeSphereFalloff, 10, FColor(155, 155, 255));
									DrawDebugSphere(World, Origin, ShapeDetails.ConeSphereRadius, 10, FColor(55, 55, 255));
								}
								else
								{
									DrawDebugSphere(World, Origin, ShapeDetails.ConeSphereRadius, 10, FColor(155, 155, 255));
								}
							}

							break;
						}

						default:
							check(false);
						}
					}
				}
			}
		}

		return Y;
	}

	int32 FAudioDebugger::RenderStatWaves(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y)
	{
		FAudioDevice* AudioDevice = nullptr;
		if (!DebugShouldRenderStat(World, Canvas, bDebugWavesForAllViewsEnabled, DebugStatNames::SoundWaves, &AudioDevice))
		{
			return Y;
		}

		const int32 FontHeight = GetStatsFont()->GetMaxCharHeight() + 2;

		FAudioStats& AudioStats = AudioDeviceStats.FindOrAdd(AudioDevice->DeviceID);
		Canvas->DrawShadowedString(X, Y, TEXT("Active Sound Waves:"), GetStatsFont(), FLinearColor::Green);
		Y += DebuggerTabWidth;

		using FWaveInstancePair = TPair<const FAudioStats::FStatWaveInstanceInfo*, const FAudioStats::FStatSoundInfo*>;
		TArray<FWaveInstancePair> WaveInstances;
		for (const FAudioStats::FStatSoundInfo& StatSoundInfo : AudioStats.StatSoundInfos)
		{
			for (const FAudioStats::FStatWaveInstanceInfo& WaveInstanceInfo : StatSoundInfo.WaveInstanceInfos)
			{
				if (WaveInstanceInfo.Volume >= MinDisplayVolume || WaveInstanceInfo.bPlayWhenSilent != 0)
				{
					WaveInstances.Emplace(&WaveInstanceInfo, &StatSoundInfo);
				}
			}
		}

		WaveInstances.Sort([](const FWaveInstancePair& A, const FWaveInstancePair& B) { return A.Key->InstanceIndex < B.Key->InstanceIndex; });

		const FColor BodyColor = GetBodyColor();
		for (const FWaveInstancePair& WaveInstanceInfo : WaveInstances)
		{
			UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(WaveInstanceInfo.Value->AudioComponentID);
			AActor* SoundOwner = AudioComponent ? AudioComponent->GetOwner() : nullptr;

			double CPUPercent = 0.;
			float RelativeRenderCost = 1.f;
			if (WaveInstanceInfo.Key->DebugInfo.IsValid())
			{
				FScopeLock Lock(&WaveInstanceInfo.Key->DebugInfo->CS);
				CPUPercent = 100. * WaveInstanceInfo.Key->DebugInfo->CPUCoreUtilization;
				RelativeRenderCost = WaveInstanceInfo.Key->DebugInfo->RelativeRenderCost;
			}
			
			FString TheString = *FString::Printf(TEXT("%4i.    %6.2f  CPU:%5.2f%% RRC:%3.2f %s   Owner: %s   SoundClass: %s"),
				WaveInstanceInfo.Key->InstanceIndex,
				WaveInstanceInfo.Key->Volume,
				CPUPercent,
				RelativeRenderCost,
				*WaveInstanceInfo.Key->WaveInstanceName.ToString(),
				SoundOwner ? *SoundOwner->GetName() : TEXT("None"),
				*WaveInstanceInfo.Key->SoundClassName.ToString());
			Canvas->DrawShadowedString(X, Y, *TheString, GetStatsFont(), WaveInstanceInfo.Key->bPlayWhenSilent == 0 ? BodyColor : FColor::Yellow);
			Y += FontHeight;
		}

		const int32 ActiveInstances = WaveInstances.Num();

		const int32 Max = AudioDevice->GetMaxChannels() / 2;
		float f = FMath::Clamp<float>((float)(ActiveInstances - Max) / (float)Max, 0.f, 1.f);
		const int32 R = FMath::TruncToInt(f * 255);

		if (ActiveInstances > Max)
		{
			f = FMath::Clamp<float>((float)(Max - ActiveInstances) / (float)Max, 0.5f, 1.f);
		}
		else
		{
			f = 1.0f;
		}
		const int32 G = FMath::TruncToInt(f * 255);
		const int32 B = 0;

		Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT(" Total: %i"), ActiveInstances), GetStatsFont(), FColor(R, G, B));
		Y += 2 * FontHeight;

		return Y;
	}

	int32 FAudioDebugger::RenderStatStreaming(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation /*= nullptr*/, const FRotator* ViewRotation /*= nullptr*/)
	{
		FAudioDevice* AudioDevice = nullptr;
		if (!DebugShouldRenderStat(World, Canvas, bDebugSoundsForAllViewsEnabled, DebugStatNames::AudioStreaming, &AudioDevice))
		{
			return Y;
		}

		return IStreamingManager::Get().GetAudioStreamingManager().RenderStatAudioStreaming(World, Viewport, Canvas, X, Y, ViewLocation, ViewRotation);
	}

	void FAudioDebugger::RemoveDevice(const FAudioDevice& AudioDevice)
	{
		AudioDeviceStats.Remove(AudioDevice.DeviceID);
		AudioDeviceStats_AudioThread.Remove(AudioDevice.DeviceID);
	}






	void FAudioDebugger::ClearStats(const FName StatsToClear, UWorld* InWorld)
	{
		if (!GEngine)
		{
			return;
		}

		if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
		{
			Audio::FAudioDebugger& DebuggerInstance = DeviceManager->GetDebugger();
			DeviceManager->IterateOverAllDevices(
				[&DebuggerInstance, World = InWorld, ClearedStats = StatsToClear](FDeviceId DeviceId, const FAudioDevice* AudioDevice)
				{
					if (!World || World->GetAudioDevice().GetDeviceID() == DeviceId)
					{
						DebuggerInstance.ClearStats(DeviceId, ClearedStats);
					}
				}
			);
		}
	}

	void FAudioDebugger::ClearStats(FDeviceId DeviceId, const FName StatToClear)
	{
		if (IsInGameThread())
		{
			FAudioStats& Stats = AudioDeviceStats.FindOrAdd(DeviceId);
			Stats.EnabledStats.Remove(StatToClear);
		}

		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ClearStats"), STAT_AudioClearStats, STATGROUP_TaskGraphTasks);

			FAudioThread::RunCommandOnAudioThread([this, DeviceId, StatToClear]()
			{
				ClearStats(DeviceId, StatToClear);
			}, GET_STATID(STAT_AudioClearStats));
			return;
		}

		FAudioStats_AudioThread& Stats = AudioDeviceStats_AudioThread.FindOrAdd(DeviceId);
		Stats.RequestedStats.Remove(StatToClear);
	}

	void FAudioDebugger::SetStats(const TSet<FName> & StatsToSet, UWorld* InWorld)
	{
		if (!GEngine)
		{
			return;
		}

		if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
		{
			Audio::FAudioDebugger& DebuggerInstance = DeviceManager->GetDebugger();
			DeviceManager->IterateOverAllDevices(
				[&DebuggerInstance, World = InWorld, StatsSet = StatsToSet](FDeviceId DeviceId, const FAudioDevice* AudioDevice)
				{
					if (!World || World->GetAudioDevice().GetDeviceID() == DeviceId)
					{
						DebuggerInstance.SetStats(DeviceId, StatsSet);
					}
				}
			);
		}
	}

	void FAudioDebugger::SetStats(FDeviceId DeviceId, const TSet<FName> & StatsToSet)
	{
		if (IsInGameThread())
		{
			FAudioStats& Stats = AudioDeviceStats.FindOrAdd(DeviceId);

			Stats.EnabledStats.Append(StatsToSet);
		}

		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetStats"), STAT_AudioSetStats, STATGROUP_TaskGraphTasks);

			FAudioThread::RunCommandOnAudioThread([this, DeviceId, StatsToSet]()
			{
				SetStats(DeviceId, StatsToSet);
			}, GET_STATID(STAT_AudioSetStats));
			return;
		}

		FAudioStats_AudioThread& Stats = AudioDeviceStats_AudioThread.FindOrAdd(DeviceId);
		
		Stats.RequestedStats.Append(StatsToSet);
	}





	bool FAudioDebugger::PostStatModulatorHelp(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		// Ignore if all Viewports are closed.
		if (!ViewportClient)
		{
			return false;
		}

		if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
		{
			if (AudioDevice->IsModulationPluginEnabled())
			{
				if (IAudioModulationManager* Modulation = AudioDevice->ModulationInterface.Get())
				{
					if (!Modulation->OnPostHelp(ViewportClient, Stream))
					{
						return false;
					}
				}
			}
		}

		return true;
	}

	void FAudioDebugger::SendUpdateResultsToGameThread(const FAudioDevice& AudioDevice, const int32 FirstActiveIndex)
	{
		check(IsInAudioThread());

		FAudioStats_AudioThread* Stats_AudioThread = AudioDeviceStats_AudioThread.Find(AudioDevice.DeviceID);
		if (!Stats_AudioThread)
		{
			return;
		}

		TArray<FAudioStats::FStatSoundInfo> StatSoundInfos;
		TArray<FAudioStats::FStatSoundMix> StatSoundMixes;

		const TSet<FName> RequestedStats = Stats_AudioThread->RequestedStats;
		TMap<FActiveSound*, int32> ActiveSoundToInfoIndex;
	
		const bool bDebug = RequestedStats.Contains(DebugStatNames::DebugSounds);

		if (RequestedStats.Contains(DebugStatNames::Sounds) || RequestedStats.Contains(DebugStatNames::SoundCues) 
			|| RequestedStats.Contains(DebugStatNames::SoundMixes) || RequestedStats.Contains(DebugStatNames::SoundWaves))
		{
			for (FActiveSound* ActiveSound : AudioDevice.GetActiveSounds())
			{
				if (USoundBase* SoundBase = ActiveSound->GetSound())
				{				
					if (!bDebug || ActiveSound->GetSound()->bDebug)
					{
						ActiveSoundToInfoIndex.Add(ActiveSound, StatSoundInfos.AddDefaulted());
						FAudioStats::FStatSoundInfo& StatSoundInfo = StatSoundInfos.Last();
						StatSoundInfo.SoundName = SoundBase->GetName();
						StatSoundInfo.SoundPath = SoundBase->GetPathName();
						StatSoundInfo.Distance = 0.f;
						if (ActiveSound->bAllowSpatialization)
						{
							StatSoundInfo.Distance = AudioDevice.GetDistanceToNearestListener(ActiveSound->Transform.GetTranslation());
						}
						StatSoundInfo.PlaybackTime = ActiveSound->PlaybackTime;
						StatSoundInfo.Priority = ActiveSound->GetConcurrencyPriority();
						StatSoundInfo.PlaybackTimeNonVirtualized = ActiveSound->PlaybackTimeNonVirtualized;
					
						StatSoundInfo.Volume = 0.0f;
						for (const TPair<UPTRINT, FWaveInstance*>& Pair : ActiveSound->GetWaveInstances())
						{
							StatSoundInfo.Volume = FMath::Max(StatSoundInfo.Volume, Pair.Value->GetVolumeWithDistanceAndOcclusionAttenuation() * Pair.Value->GetDynamicVolume());
						}

						if (USoundClass* SoundClass = ActiveSound->GetSoundClass())
						{
							StatSoundInfo.SoundClassName = SoundClass->GetFName();

						}
						else
						{
							StatSoundInfo.SoundClassName = NAME_None;
						}

						StatSoundInfo.Transform = ActiveSound->Transform;
						StatSoundInfo.AudioComponentID = ActiveSound->GetAudioComponentID();

						if (bAttenuationVisualizeEnabledCVar && ActiveSound->GetSound()->bDebug)
						{
							ActiveSound->CollectAttenuationShapesForVisualization(StatSoundInfo.ShapeDetailsMap);
						}
					}
				}
			}

			// Iterate through all wave instances.
			const TArray<FWaveInstance*>& WaveInstances = AudioDevice.GetActiveWaveInstances();
			auto WaveInstanceSourceMap = AudioDevice.GetWaveInstanceSourceMap();
			for (int32 InstanceIndex = FirstActiveIndex; InstanceIndex < WaveInstances.Num(); ++InstanceIndex)
			{
				const FWaveInstance* WaveInstance = WaveInstances[InstanceIndex];
				const FActiveSound* ActiveSound = WaveInstance->ActiveSound;
				check(ActiveSound);

				if (const int32* SoundInfoIndex = ActiveSoundToInfoIndex.Find(ActiveSound))
				{
					const USoundClass* SoundClass = ActiveSound->GetSoundClass();
					if (const USoundClass* WaveSoundClass = WaveInstance->SoundClass)
					{
						SoundClass = WaveSoundClass;
					}

					FAudioStats::FStatWaveInstanceInfo WaveInstanceInfo;
					FSoundSource* Source = WaveInstanceSourceMap.FindRef(WaveInstance);
					WaveInstanceInfo.Description = Source ? Source->Describe(RequestedStats.Contains(DebugStatNames::LongSoundNames)) : FString("No source");
					WaveInstanceInfo.Volume = WaveInstance->GetVolumeWithDistanceAndOcclusionAttenuation() * WaveInstance->GetDynamicVolume();
					WaveInstanceInfo.InstanceIndex = InstanceIndex;
					WaveInstanceInfo.WaveInstanceName = *WaveInstance->GetName();
					WaveInstanceInfo.bPlayWhenSilent = ActiveSound->IsPlayWhenSilent() ? 1 : 0;
					WaveInstanceInfo.DebugInfo = Source ? Source->DebugInfo : WaveInstanceInfo.DebugInfo;
					WaveInstanceInfo.SoundClassName = SoundClass ? SoundClass->GetFName() : NAME_None;
					StatSoundInfos[*SoundInfoIndex].WaveInstanceInfos.Add(MoveTemp(WaveInstanceInfo));
				}
			}
		}

		if (RequestedStats.Contains(DebugStatNames::SoundMixes))
		{
			if (const FAudioEffectsManager* Effects = AudioDevice.GetEffects())
			{
				const USoundMix* CurrentEQMix = Effects->GetCurrentEQMix();

				for (const TPair<USoundMix*, FSoundMixState>& SoundMixPair : AudioDevice.GetSoundMixModifiers())
				{
					StatSoundMixes.AddDefaulted();
					FAudioStats::FStatSoundMix& StatSoundMix = StatSoundMixes.Last();
					StatSoundMix.MixName = SoundMixPair.Key->GetName();
					StatSoundMix.InterpValue = SoundMixPair.Value.InterpValue;
					StatSoundMix.RefCount = SoundMixPair.Value.ActiveRefCount + SoundMixPair.Value.PassiveRefCount;
					StatSoundMix.bIsCurrentEQ = (SoundMixPair.Key == CurrentEQMix);
				}
			}
		}

		DECLARE_CYCLE_STAT(TEXT("FGameThreadAudioTask.AudioSendResults"), STAT_AudioSendResults, STATGROUP_TaskGraphTasks);

		const uint32 AudioDeviceID = AudioDevice.DeviceID;

		TArray<FTransform> ListenerTransforms;
		for (const FListener& Listener : AudioDevice.GetListeners())
		{
			ListenerTransforms.Add(Listener.Transform);
		}
		FAudioThread::RunCommandOnGameThread([AudioDeviceID, ListenerTransforms, StatSoundInfos, StatSoundMixes]()
		{
			FAudioStats& Stats = AudioDeviceStats.FindOrAdd(AudioDeviceID);
			Stats.ListenerTransforms = ListenerTransforms;
			Stats.StatSoundInfos = StatSoundInfos;
			Stats.StatSoundMixes = StatSoundMixes;
		}, GET_STATID(STAT_AudioSendResults));
	}

	void FAudioDebugger::UpdateAudibleInactiveSounds(const uint32 FirstActiveIndex, const TArray<FWaveInstance*>& WaveInstances)
	{
	#if STATS
		uint32 AudibleInactiveSounds = 0;
		// Count how many sounds are not being played but were audible
		for (uint32 InstanceIndex = 0; InstanceIndex < FirstActiveIndex; ++InstanceIndex)
		{
			const FWaveInstance* WaveInstance = WaveInstances[InstanceIndex];
			const float WaveInstanceVol = WaveInstance->GetVolumeWithDistanceAndOcclusionAttenuation() * WaveInstance->GetDynamicVolume();
			if (WaveInstanceVol > MinDisplayVolume)
			{
				AudibleInactiveSounds++;
			}
		}
		SET_DWORD_STAT(STAT_AudibleWavesDroppedDueToPriority, AudibleInactiveSounds);
	#endif
	}

	void FAudioDebugger::ClearMutesAndSolos()
	{
		ExplicitlySoloedSubmixes.Reset();
		ExplicitlyMutedSubmixes.Reset();

		ExecuteCmdOnAudioThread([this]
		{
			DebugNames.MuteSoundClass.Empty();
			DebugNames.MuteSoundCue.Empty();
			DebugNames.MuteSoundWave.Empty();
			DebugNames.MuteSubmix.Empty();
			DebugNames.SoloSoundClass.Empty();
			DebugNames.SoloSoundCue.Empty();
			DebugNames.SoloSoundWave.Empty();
			DebugNames.SoloSubmix.Empty();
		});
	}

	bool FAudioDebugger::IsMuteOrSoloActive() const
	{
		return !DebugNames.MuteSoundClass.IsEmpty()
			|| !DebugNames.MuteSoundCue.IsEmpty()
			|| !DebugNames.MuteSoundWave.IsEmpty()
			|| !DebugNames.MuteSubmix.IsEmpty()
			|| !DebugNames.SoloSoundClass.IsEmpty()
			|| !DebugNames.SoloSoundCue.IsEmpty()
			|| !DebugNames.SoloSoundWave.IsEmpty()
			|| !DebugNames.SoloSubmix.IsEmpty();
	}

	bool FAudioDebugger::IsSubmixMuteOrSoloActive() const
	{
		return !DebugNames.SoloSubmix.IsEmpty()
			|| !DebugNames.MuteSubmix.IsEmpty();
	}

	void FAudioDebugger::ToggleSoloSubmixChain(const FName InSubmixName)
	{
		ensure(IsInGameThread());

		SetSoloSubmixChain(InSubmixName, !IsSubmixExplicitlySoloed(InSubmixName));
	}

	void FAudioDebugger::SetSoloSubmixChain(const FName InSubmixName, const bool bInSolo)
	{
		ensure(IsInGameThread());

		if (bInSolo)
		{
			ExplicitlySoloedSubmixes.Add(InSubmixName);
		}
		else
		{
			ExplicitlySoloedSubmixes.Remove(InSubmixName);
		}

		RefreshComputedSubmixMuteSoloChain();
	}

	void FAudioDebugger::ToggleMuteSubmix(const FName InSubmixName)
	{
		ensure(IsInGameThread());

		SetMuteSubmix(InSubmixName, !IsSubmixExplicitlyMuted(InSubmixName));
	}

	void FAudioDebugger::SetMuteSubmix(const FName InSubmixName, const bool bInMute)
	{
		ensure(IsInGameThread());

		if (bInMute)
		{
			ExplicitlyMutedSubmixes.Add(InSubmixName);
		}
		else
		{
			ExplicitlyMutedSubmixes.Remove(InSubmixName);
		}

		RefreshComputedSubmixMuteSoloChain();
	}

	void FAudioDebugger::ClearSubmixMuteSoloState()
	{
		ensure(IsInGameThread());

		ExplicitlySoloedSubmixes.Reset();
		ExplicitlyMutedSubmixes.Reset();

		ExecuteCmdOnAudioThread([this]
		{
			DebugNames.SoloSubmix.Empty();
			DebugNames.MuteSubmix.Empty();
		});
	}

	bool FAudioDebugger::IsSubmixExplicitlySoloed(const FName InSubmixName) const
	{
		ensure(IsInGameThread());

		return ExplicitlySoloedSubmixes.Contains(InSubmixName);
	}

	bool FAudioDebugger::IsSubmixExplicitlyMuted(const FName InSubmixName) const
	{
		ensure(IsInGameThread());

		return ExplicitlyMutedSubmixes.Contains(InSubmixName);
	}

	void FAudioDebugger::CollectSubmixHierarchyNames(const TObjectPtr<USoundSubmix>& InSubmix,
													 const TObjectPtr<USoundSubmix>& InMainSubmix,
													 const TObjectPtr<USoundSubmix>& InBaseDefaultSubmix,
													 TSet<FName>& OutNames) const
	{
		if (InSubmix == nullptr)
		{
			return;
		}

		// Walk up via ParentSubmix to collect ancestors.
		// When an orphan submix (nullptr parent, non-dynamic) is encountered,
		// continue to BaseDefault (or Main if no BaseDefault) to mirror
		// the implicit routing the mixer applies at runtime.
		TObjectPtr<USoundSubmixBase> Ancestor = InSubmix;
		while (Ancestor)
		{
			const FName AncestorName = Ancestor->GetFName();

			bool bAlreadyInSet = false;
			OutNames.Add(AncestorName, &bAlreadyInSet);

			if (bAlreadyInSet)
			{
				break;
			}

			const TObjectPtr<USoundSubmixWithParentBase> Parented = Cast<USoundSubmixWithParentBase>(Ancestor);
			if (Parented == nullptr)
			{
				break;
			}

			// Move to the parent submix if one is defined explicitly
			if (Parented->ParentSubmix)
			{
				Ancestor = Parented->ParentSubmix;
			}
			// If no parent is defined explicitly, audio mixer will implicitly route the
			// submix to the base default submix (if defined) or the main submix
			// 
			// Early out for dynamic submixes - the mixer explicitly skips auto-connecting these
			// as they only connect via DynamicConnect() - if no parent is supplied they are considered
			// intentionally disconnected
			else if (Parented->IsDynamic(true))
			{
				break;
			}
			// Base default submix takes priority over main submix if project defines one
			else if (InBaseDefaultSubmix && Ancestor != InBaseDefaultSubmix)
			{
				Ancestor = InBaseDefaultSubmix;
			}
			else if (InMainSubmix && Ancestor != InMainSubmix)
			{
				Ancestor = InMainSubmix;
			}
			// No parent - end of chain
			else
			{
				break;
			}
		}

		// Walk down via ChildSubmixes (DFS) to collect descendants
		TArray<TObjectPtr<USoundSubmixBase>> Stack;
		Stack.Add(InSubmix);

		while (Stack.Num() > 0)
		{
			const TObjectPtr<USoundSubmixBase> Node = Stack.Pop();
			for (const TObjectPtr<USoundSubmixBase>& Child : Node->ChildSubmixes)
			{
				if (Child == nullptr)
				{
					continue;
				}

				bool bAlreadyInSet = false;
				OutNames.Add(Child->GetFName(), &bAlreadyInSet);

				if (!bAlreadyInSet)
				{
					Stack.Add(Child);
				}
			}
		}
	}

	void FAudioDebugger::RefreshComputedSubmixMuteSoloChain()
	{
		// Grab the main submix and base default submix (if defined)
		const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();

		TObjectPtr<USoundSubmix> MainSubmix = nullptr;
		TObjectPtr<USoundSubmix> BaseDefaultSubmix = nullptr;

		if (AudioSettings != nullptr)
		{
			MainSubmix = Cast<USoundSubmix>(AudioSettings->MasterSubmix.TryLoad());

			if (AudioSettings->BaseDefaultSubmix.IsValid())
			{
				BaseDefaultSubmix = Cast<USoundSubmix>(AudioSettings->BaseDefaultSubmix.TryLoad());
			}
		}

		TSet<FName> ComputedSoloNames;
		ProcessExplicitlySoloedSubmixChains(MainSubmix, BaseDefaultSubmix, ComputedSoloNames);
		ProcessExplicitlySoloedMainSubmixChains(MainSubmix, BaseDefaultSubmix, ComputedSoloNames);

		TSet<FName> ComputedMuteNames = ExplicitlyMutedSubmixes.Difference(ExplicitlySoloedSubmixes);

		ExecuteCmdOnAudioThread([this, ComputedSoloNames = MoveTemp(ComputedSoloNames), ComputedMuteNames = MoveTemp(ComputedMuteNames)]
		{
			DebugNames.SoloSubmix = ComputedSoloNames.Array();
			DebugNames.MuteSubmix = ComputedMuteNames.Array();
		});
	}

	void FAudioDebugger::ProcessExplicitlySoloedSubmixChains(const TObjectPtr<USoundSubmix>& InMainSubmix, const TObjectPtr<USoundSubmix>& InBaseDefaultSubmix, TSet<FName>& OutComputedSoloNames)
	{
		// Iterate through the submixes that have been explicitly soloed.
		// For each one - collect the names of submixes in their hierarchy
		// chain and add them to ComputedSoloNames.
		for (const FName& SoloedName : ExplicitlySoloedSubmixes)
		{
			const TObjectPtr<USoundSubmix> Submix = FindSubmixByName(SoloedName);
			if (Submix == nullptr)
			{
				continue;
			}

			// Submixes that have been explicitly soloed are always soloed
			OutComputedSoloNames.Add(SoloedName);

			AddImplicitSolosFromHierarchy(Submix, InMainSubmix, InBaseDefaultSubmix, OutComputedSoloNames);
		}
	}

	void FAudioDebugger::ProcessExplicitlySoloedMainSubmixChains(const TObjectPtr<USoundSubmix>& InMainSubmix, const TObjectPtr<USoundSubmix>& InBaseDefaultSubmix, TSet<FName>& OutComputedSoloNames)
	{
		// When Main or BaseDefault is directly soloed, also collect orphan submixes
		// (no explicit parent, non-dynamic) that implicitly route there at runtime,
		// and walk their full descendant chains.

		const FName MainSubmixName = InMainSubmix ? InMainSubmix->GetFName() : NAME_None;
		const FName BaseDefaultSubmixName = InBaseDefaultSubmix ? InBaseDefaultSubmix->GetFName() : NAME_None;

		const bool bExplicitlySoloedMainOrBaseDefault = ExplicitlySoloedSubmixes.Contains(MainSubmixName)
														|| ExplicitlySoloedSubmixes.Contains(BaseDefaultSubmixName);

		if (!bExplicitlySoloedMainOrBaseDefault)
		{
			return;
		}

		// At least one of the main submixes is explicitly soloed
		// Find all submixes with no linked parent and implicitly link them (mirroring audio mixer behaviour)
		for (TObjectIterator<USoundSubmix> It; It; ++It)
		{
			const TObjectPtr<USoundSubmix> Submix = *It;
			if (Submix == nullptr || OutComputedSoloNames.Contains(Submix->GetFName()))
			{
				continue;
			}

			// If the submix is dynamic or has an explicit parent defined, skip
			if (Submix->ParentSubmix != nullptr || Submix->IsDynamic(true))
			{
				continue;
			}

			const FName SubmixName = Submix->GetFName();

			// If the base default submix is defined, and this is not the base default submix, this submix will route to the base default
			const bool bRoutesToBaseDefault = BaseDefaultSubmixName != NAME_None && SubmixName != BaseDefaultSubmixName;
			const FName ImplicitParentName = bRoutesToBaseDefault ? BaseDefaultSubmixName : MainSubmixName;

			// If the implicit submix we are linked to (base or main) is not soloed, skip
			if (!OutComputedSoloNames.Contains(ImplicitParentName))
			{
				continue;
			}

			AddImplicitSolosFromHierarchy(Submix, InMainSubmix, InBaseDefaultSubmix, OutComputedSoloNames);
		}
	}

	void FAudioDebugger::AddImplicitSolosFromHierarchy(const TObjectPtr<USoundSubmix>& InSubmix, const TObjectPtr<USoundSubmix>& InMainSubmix, const TObjectPtr<USoundSubmix>& InBaseDefaultSubmix, TSet<FName>& OutComputedSoloNames)
	{
		// Grab the submix chain connected to this submix
		// The additional submixes in this chain will be
		// implicitly soloed
		TSet<FName> HierarchyNames;
		CollectSubmixHierarchyNames(InSubmix, InMainSubmix, InBaseDefaultSubmix, HierarchyNames);

		for (const FName& Name : HierarchyNames)
		{
			// Explicit mute of a submix takes priority over implicit solo
			// Explicit solo takes priority over explicit mute
			if (!ExplicitlyMutedSubmixes.Contains(Name) || ExplicitlySoloedSubmixes.Contains(Name))
			{
				OutComputedSoloNames.Add(Name);
			}
		}
	}

	void FAudioDebugger::LogSubtitle(const TCHAR* InCmd, USoundWave& InSoundWave)
	{
		const bool bLogSubtitle = FParse::Param(InCmd, TEXT("LogSubtitle"));
		if (bLogSubtitle)
		{
			FString Subtitle;
			for (int32 i = 0; i < InSoundWave.Subtitles.Num(); i++)
			{
				Subtitle += InSoundWave.Subtitles[i].Text.ToString();
			}

			if (Subtitle.Len() == 0)
			{
				Subtitle = InSoundWave.SpokenText_DEPRECATED;
			}

			if (Subtitle.Len() == 0)
			{
				Subtitle = "<NO SUBTITLE>";
			}

			UE_LOGF(LogAudio, Display, "Subtitle:  %ls", *Subtitle);
#if WITH_EDITORONLY_DATA
			UE_LOGF(LogAudio, Display, "Comment:   %ls", *InSoundWave.Comment);
#endif // WITH_EDITORONLY_DATA
			UE_LOGF(LogAudio, Display, "Mature:    %ls", InSoundWave.bMature ? TEXT("Yes") : TEXT("No"));
		}
	}
} // namespace Audio
#endif // ENABLE_AUDIO_DEBUG
