// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioDecompress.h"
#include "AudioDevice.h"
#include "AudioStreamingCache.h"
#include "Engine/Engine.h"
#include "Hash/xxhash.h"
#include "IAudioDecoderOverride.h"
#include "Interfaces/IAudioFormat.h"
#include "Misc/CoreStats.h"
#include "Misc/ScopeRWLock.h"
#include "Stats/StatsTrace.h"
#include "Sound/StreamedAudioChunkSeekTable.h"

namespace AudioDecompressPrivate
{
#ifndef WITH_AUDIO_DECODER_DIAGNOSTICS
	// Optionally enable this manually with a flag. They use strings, so shouldn't be enabled by default.
	#define WITH_AUDIO_DECODER_DIAGNOSTICS (0)
#endif //WITH_AUDIO_DECODER_DIAGNOSTICS

	static int32 bEnableRetainingNextPlayingChunkCVar = 0;
	FAutoConsoleVariableRef CVarEnableRetainingNextPlayingChunk(
		TEXT("au.streamcache.RetainNextChunk"),
		bEnableRetainingNextPlayingChunkCVar,
		TEXT("Keep a reference to next to play chunk" ),
		ECVF_Default);
	


#if WITH_AUDIO_DECODER_DIAGNOSTICS
	
	static FString ForceDecoderErrorOnWaveCVar;
	FAutoConsoleVariableRef CVarForceDecoderErrorOnWave(
		TEXT("au.debug.force_decoder_error_on_wave"),
		ForceDecoderErrorOnWaveCVar,
		TEXT("Force Decoder Error On Any decoding wave matching this string."),
		ECVF_Default);

	static FString ForceDecoderNegativeSamplesOnWaveCVar;
	FAutoConsoleVariableRef CVarForceDecoderNegativeSamplesOnWave(
		TEXT("au.debug.force_decoder_negative_samples_on_wave"),
		ForceDecoderNegativeSamplesOnWaveCVar,
		TEXT("Force Negative Samples on Decode call to simulate error" ),
		ECVF_Default);

	// Macro to string match against the debugging wave of choice.
	#define DECODER_MATCHES_WAVE(STR)\
		(!STR.IsEmpty() && StreamingSoundWaveData.IsValid() && StreamingSoundWaveData->GetFName().ToString().Contains(STR))
	
#else  //WITH_AUDIO_DECODER_DIAGNOSTICS
	#define DECODER_MATCHES_WAVE(STR) (false)
#endif //WITH_AUDIO_DECODER_DIAGNOSTICS
}

int32 ICompressedAudioInfo::ConformDecodeSize(int32 InMaxDesiredDecodeSizeInFrames)
{
	/** Minimum number of frames to decode per a call to the decoder.  */
	static constexpr int32 DefaultMinDecodeSizeInFrames = 128;

	/** Some codecs have strict requirements on decode size. In order to be
		* functional with all supported codecs, the decode size must be a multiple
		* of 128.
		*/
	static constexpr int32 DecodeSizeQuantizationInFrames = 128;

	static_assert(DefaultMinDecodeSizeInFrames >= DecodeSizeQuantizationInFrames, "Min decode size less than decode size quantization");
	static_assert((DefaultMinDecodeSizeInFrames % DecodeSizeQuantizationInFrames) == 0, "Min decode size must be equally divisible by decode size quantization");

	InMaxDesiredDecodeSizeInFrames = FMath::Max(InMaxDesiredDecodeSizeInFrames, DefaultMinDecodeSizeInFrames);
	const int32 QuantizedDecodeSizeInFrames = (InMaxDesiredDecodeSizeInFrames / DecodeSizeQuantizationInFrames) * DecodeSizeQuantizationInFrames;
	const int32 ConformedDecodeSizeInFrames = FMath::Max(QuantizedDecodeSizeInFrames, DefaultMinDecodeSizeInFrames);

	check((ConformedDecodeSizeInFrames % QuantizedDecodeSizeInFrames) == 0);
	check(ConformedDecodeSizeInFrames > 0);

	return ConformedDecodeSizeInFrames;
}

IStreamedCompressedInfo::IStreamedCompressedInfo()
	: bIsStreaming(false)
	, SrcBufferData(nullptr)
	, SrcBufferDataSize(0)
	, SrcBufferOffset(0)
	, AudioDataOffset(0)
	, AudioDataChunkIndex(0)
	, TrueSampleCount(0)
	, CurrentSampleCount(0)
	, NumChannels(0)
	, MaxFrameSizeSamples(0)
	, SampleStride(0)
	, LastPCMByteSize(0)
	, LastPCMOffset(0)
	, bStoringEndOfFile(false)
	, CurrentChunkIndex(0)
	, SrcBufferPadding(0)
	, StreamSeekBlockIndex(INDEX_NONE)
	, StreamSeekBlockOffset(0)
{
}

IStreamedCompressedInfo::~IStreamedCompressedInfo() = default;

uint32 IStreamedCompressedInfo::Read(void *OutBuffer, uint32 DataSize)
{
	uint32 BytesToRead = FMath::Min(DataSize, SrcBufferDataSize - SrcBufferOffset);
	if (BytesToRead > 0)
	{
		FMemory::Memcpy(OutBuffer, SrcBufferData + SrcBufferOffset, BytesToRead);
		SrcBufferOffset += BytesToRead;
	}
	return BytesToRead;
}

bool IStreamedCompressedInfo::ReadCompressedInfo(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo)
{
	check(!SrcBufferData);

	SCOPE_CYCLE_COUNTER(STAT_AudioStreamedDecompressTime);

	// Parse the format header, this is done different for each format
	if (!ParseHeader(InSrcBufferData, InSrcBufferDataSize, QualityInfo))
	{
		UE_LOGF(LogAudio, Error, "IStreamedCompressedInfo::ReadCompressedInfo: Failed to Parse Header");
		bHasError = true;
		return false;
	}

	// After parsing the header, the SrcBufferData should be none-null
	check(SrcBufferData != nullptr);

	// Sample Stride is 
	SampleStride = NumChannels * MONO_PCM_SAMPLE_SIZE;

	MaxFrameSizeSamples = GetMaxFrameSizeSamples();
		
	LastDecodedPCM.Empty(MaxFrameSizeSamples * SampleStride);
	LastDecodedPCM.AddUninitialized(MaxFrameSizeSamples * SampleStride);

	return CreateDecoder();
}

bool IStreamedCompressedInfo::ReadCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize)
{
	check(Destination);

	SCOPE_CYCLE_COUNTER(STAT_AudioStreamedDecompressTime);

	bool bFinished = false;
	uint32 TotalBytesDecoded = 0;

	while (TotalBytesDecoded < BufferSize)
	{
		const uint8* EncodedSrcPtr = SrcBufferData + SrcBufferOffset;
		const uint32 RemainingEncodedSrcSize = SrcBufferDataSize - SrcBufferOffset;

		const FDecodeResult DecodeResult = Decode(EncodedSrcPtr, RemainingEncodedSrcSize, Destination, BufferSize - TotalBytesDecoded);
		if (!DecodeResult.NumPcmBytesProduced)
		{
			bFinished = true;

			if (bLooping)
			{
				SrcBufferOffset = AudioDataOffset;
				CurrentSampleCount = 0;

				PrepareToLoop();
			}
			else
			{
				// Zero the remainder of the buffer
				FMemory::Memzero(Destination, BufferSize - TotalBytesDecoded);
				break;
			}
		}
		else if (DecodeResult.NumPcmBytesProduced < 0)
		{
			// Zero the remainder of the buffer
			FMemory::Memzero(Destination, BufferSize - TotalBytesDecoded);
			return true;
		}

		TotalBytesDecoded += DecodeResult.NumPcmBytesProduced;
		SrcBufferOffset += DecodeResult.NumCompressedBytesConsumed;
		Destination += DecodeResult.NumPcmBytesProduced;
	}

	return bFinished;
}

void IStreamedCompressedInfo::SeekToTime(const float InSeekTimeSeconds)
{
	if (!StreamingSoundWaveData.IsValid())
	{
		return;
	}

	const TSharedRef<const FSoundWaveData> WaveData = StreamingSoundWaveData.ToSharedRef();

	// Negative time will seek to start.
	int64 SeekTimeAudioFrames = 0;
	if (InSeekTimeSeconds > 0)
	{
		SeekTimeAudioFrames = FMath::FloorToInt64(WaveData->GetSampleRate() * InSeekTimeSeconds);
		if (!IntFitsIn<uint32>(SeekTimeAudioFrames))
		{
			UE_LOGF(LogAudio, Warning, "Seek too large (%.2f seconds), ignoring...", InSeekTimeSeconds);
			return;
		}
	}

	// If we have a chunk setup to contain a seek-table it will return a value other than INDEX_NONE here.
	const int32 ChunkIndexToSeekTo = WaveData->FindChunkIndexForSeeking(IntCastChecked<uint32>(SeekTimeAudioFrames));
	if (ChunkIndexToSeekTo >= 0)
	{
		StreamSeekBlockIndex = ChunkIndexToSeekTo;
		StreamSeekBlockOffset = 0;								// We don't know this until the seek-table is loaded, so we leave it 0 for now.
		StreamSeekToAudioFrames = SeekTimeAudioFrames;			// Store the time in frames so when we load the chunks table loads we can find the offset.
	}
}

void IStreamedCompressedInfo::SeekToFrame(const uint32 InSeekTimeFrames)
{
	if (!StreamingSoundWaveData.IsValid())
	{
		return;
	}

	const TSharedRef<const FSoundWaveData> WaveData = StreamingSoundWaveData.ToSharedRef();

	// If we have a chunk setup to contain a seek-table it will return a value other than INDEX_NONE here.
	const int32 ChunkIndexToSeekTo = WaveData->FindChunkIndexForSeeking(IntCastChecked<uint32>(InSeekTimeFrames));
	if (ChunkIndexToSeekTo >= 0)
	{
		StreamSeekBlockIndex = ChunkIndexToSeekTo;
		StreamSeekBlockOffset = 0;								// We don't know this until the seek-table is loaded, so we leave it 0 for now.
		StreamSeekToAudioFrames = InSeekTimeFrames;			// Store the time in frames so when we load the chunks table loads we can find the offset.
	}
}

void IStreamedCompressedInfo::ExpandFile(uint8* DstBuffer, struct FSoundQualityInfo* QualityInfo)
{
	check(DstBuffer);
	check(QualityInfo);

	// Ensure we're at the start of the audio data
	SrcBufferOffset = AudioDataOffset;

	uint32 RawPCMOffset = 0;

	while (RawPCMOffset < QualityInfo->SampleDataSize)
	{
		const int32 DecodedFrames = DecompressToPCMBuffer( /*Unused*/ 0);

		if (DecodedFrames < 0)
		{
			RawPCMOffset += ZeroBuffer(DstBuffer + RawPCMOffset, QualityInfo->SampleDataSize - RawPCMOffset);
		}
		else
		{
			LastPCMByteSize = IncrementCurrentSampleCount(DecodedFrames * NumChannels) * MONO_PCM_SAMPLE_SIZE;
			RawPCMOffset += WriteFromDecodedPCM(DstBuffer + RawPCMOffset, QualityInfo->SampleDataSize - RawPCMOffset);
		}
	}
}

bool IStreamedCompressedInfo::StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo)
{
	// we have already cached the wave, confirm it is valid and the same
	if (!(ensureAlways(StreamingSoundWaveData.IsValid() && (StreamingSoundWaveData == InWaveData))))
	{
		return false;
	}

	// Get the zeroth chunk of data (should always be loaded)
	CurrentChunkIndex = 0;
	uint32 ChunkSize = 0;
	const uint8* ZerothChunk = GetLoadedChunk(StreamingSoundWaveData.ToSharedRef(), CurrentChunkIndex, ChunkSize);

	// If there is a seek-table present in the chunk, we'll need to parse that first.
	if (InWaveData->HasChunkSeekTable(CurrentChunkIndex))
	{
		uint32 SeekTableEnd = 0;
		if (ensureMsgf(FStreamedAudioChunkSeekTable::Parse(ZerothChunk, ChunkSize, SeekTableEnd, GetCurrentSeekTable()),
			TEXT("Failed to parse seektable in '%s' chunk=%d"), *StreamingSoundWaveData->GetFName().ToString(), CurrentChunkIndex))
		{
			ChunkSize -= SeekTableEnd;
			ZerothChunk += SeekTableEnd;
		}
	}
	
	bIsStreaming = true;
	if (ZerothChunk)
	{
		bool HeaderReadResult = ReadCompressedInfo(ZerothChunk, ChunkSize, QualityInfo);
		
		// If we've read through the entirety of the zeroth chunk while parsing the header, move onto the next chunk.
		if (SrcBufferOffset >= ChunkSize)
		{
			CurrentChunkIndex++;
			SrcBufferData = NULL;
			SrcBufferDataSize = 0;

			AudioDataChunkIndex = CurrentChunkIndex;
			AudioDataOffset -= ChunkSize;
		}

		return HeaderReadResult;
	}

	return false;
}

ICompressedAudioInfo::ICompressedAudioInfo()
	: StreamingSoundWaveData(nullptr)
{}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ICompressedAudioInfo::~ICompressedAudioInfo() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool ICompressedAudioInfo::HasError() const
{
	return bHasError || DECODER_MATCHES_WAVE(AudioDecompressPrivate::ForceDecoderErrorOnWaveCVar);
}

bool ICompressedAudioInfo::StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo)
{
	UE_LOG(LogAudio, Error, TEXT("This virtual function must be overriden, and will become pure virtual in a future release"));
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<const FSoundWaveProxy> WaveProxy = FSoundWaveProxy::CreateNonUpdatingCopy(InWaveData).ToSharedPtr();
	return StreamCompressedInfoInternal(WaveProxy, QualityInfo);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool IStreamedCompressedInfo::StreamCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed)
{
	check(Destination);
	check(StreamingSoundWaveData);
	SCOPED_NAMED_EVENT(IStreamedCompressedInfo_StreamCompressedData, FColor::Blue);

	SCOPE_CYCLE_COUNTER(STAT_AudioStreamedDecompressTime);
	
	UE_LOGF(LogAudioStreamCaching, VeryVerbose, "Streaming compressed data from SoundWave'%ls' - Chunk=%d\tCurrentSampleCount=%d\tTrueSampleCount=%d\tNumChunks=%d\tOffset=%d\tChunkSize=%d\tLooping=%ls\tLastPCMOffset=%d\tContainsEOF=%ls", 
		*StreamingSoundWaveData->GetFName().ToString(), 	
		CurrentChunkIndex, 
		CurrentSampleCount, 
		TrueSampleCount, 
		StreamingSoundWaveData->GetNumChunks(),
		SrcBufferOffset, 
		SrcBufferDataSize,
		bLooping ? TEXT("YES") : TEXT("NO"),
		LastPCMOffset,	
		bStoringEndOfFile ? TEXT("YES") : TEXT("NO")
	);

	TSharedRef<const FSoundWaveData> SoundWaveDataRef = StreamingSoundWaveData.ToSharedRef();

	// If we have a pending next chunk from seeking, move to it now.
	if (StreamSeekBlockIndex != INDEX_NONE)
	{
		UE_CLOG(WantsFilteredLogging(), LogAudioStreamCaching, Display, TEXT("[%s.%p] - Seek Requesting Block = %d"), *WriteToString<128>(SoundWaveDataRef->GetFName()), this, StreamSeekBlockIndex.load(std::memory_order_relaxed));

		uint32 ChunkSize = 0;
		const uint8* NewlySeekedChunk = GetLoadedChunk(SoundWaveDataRef, StreamSeekBlockIndex, ChunkSize); 
		UE_LOGF(LogAudio, VeryVerbose, "Seek request for (%ls): (%ls) Chunk=%d (%ls), Offset=%d, OffsetInAudioFrames=%u",
			*SoundWaveDataRef->GetFName().ToString(),
			StreamSeekToAudioFrames != INDEX_NONE ? TEXT("Using streaming seek-tables") : TEXT("Using chunk/offset pair"),
			StreamSeekBlockIndex.load(),
			NewlySeekedChunk != nullptr ? TEXT("cache hit") : TEXT("cache miss"),
			StreamSeekBlockOffset,
			StreamSeekToAudioFrames
		);
		
		if (NewlySeekedChunk == nullptr)
		{
			// After a seek we're likely to need to wait a bit for the chunk to get in to memory.
			ZeroBuffer(Destination, BufferSize);
			return false;
		}	

		if (StreamSeekToAudioFrames == INDEX_NONE)
		{
			// Non-streaming-seek tables.
			// Commit the new chunk as the current.
			SrcBufferData = NewlySeekedChunk;
			CurrentChunkIndex = StreamSeekBlockIndex;
			SrcBufferDataSize = ChunkSize;
			SrcBufferOffset = StreamSeekBlockOffset;
			LastPCMByteSize = 0;
			LastPCMOffset = 0;
			bStoringEndOfFile = false;
		}
		else
		{
			// If we are using a chunked seek-table, we need to parse the table first.
			if (SoundWaveDataRef->HasChunkSeekTable(StreamSeekBlockIndex))
			{
				uint32 TableOffset = 0;
				if (ensureMsgf(FStreamedAudioChunkSeekTable::Parse(NewlySeekedChunk, ChunkSize, TableOffset, GetCurrentSeekTable()),
							   TEXT("Failed to parse seektable in '%s', chunk=%d"), *SoundWaveDataRef->GetFName().ToString(), StreamSeekBlockIndex.load(std::memory_order_relaxed)))
				{
					if (StreamSeekToAudioFrames != INDEX_NONE)
					{
						const uint32 MaxSeekableFrame = (TrueSampleCount / FMath::Max((uint32)NumChannels, uint32(1))) - 1;
						StreamSeekToAudioFrames = FMath::Min(StreamSeekToAudioFrames, MaxSeekableFrame); // protect against seek past end of audio.
						const FStreamedAudioChunkSeekTable& SeekTable = GetCurrentSeekTable();

						uint32 Offset = SeekTable.FindOffset(StreamSeekToAudioFrames);
						if (Offset != INDEX_NONE)
						{
							uint32 ActualSeekFrame = SeekTable.FindTime(Offset);
							if (ActualSeekFrame == INDEX_NONE)
							{
								// Fallback to requested frame position if the actual frame could not be found.
								ActualSeekFrame = StreamSeekToAudioFrames;
								UE_LOGF(LogAudioStreamCaching, Warning, "Failed to seek to frame %d in sound %ls due to bad seek table", StreamSeekToAudioFrames, *SoundWaveDataRef->GetFName().ToString());
							}
							// All looks good, commit this as our current chunk.
							CurrentChunkIndex = StreamSeekBlockIndex;
							SrcBufferDataSize = ChunkSize;
							SrcBufferData = NewlySeekedChunk;

							// Set our offset from the table.
							SrcBufferOffset = Offset + TableOffset;
							SrcBufferOffset += CurrentChunkIndex == 0 ? AudioDataOffset : 0;

							// Convert frames to samples and update "current sample count"
							CurrentSampleCount = FMath::Min(ActualSeekFrame * NumChannels, TrueSampleCount); // Safeguard against corrupt seektable or bad implementation
							LastPCMByteSize = 0;
							LastPCMOffset = 0;
							bStoringEndOfFile = false;

							UE_CLOG(WantsFilteredLogging(), LogAudioStreamCaching, Display, TEXT("[%s.%p] - Committing Seek Chunk=%d Size=%u"), *WriteToString<128>(SoundWaveDataRef->GetFName()), this, CurrentChunkIndex, ChunkSize);
						}
						else // Seek failed (off the end of the chunk).
						{
							const float TimeInSeconds = (float)StreamSeekToAudioFrames / SoundWaveDataRef->GetSampleRate();
							UE_LOGF(LogAudio, Log, "Failed seeking to %2.2f seconds as it's off the end of the stream. Wave=%ls", TimeInSeconds, *SoundWaveDataRef->GetFName().ToString());
						}
					}
				}
			}
		}

		// Chunk seeking logic over, turn off seeking state. 
		// StreamSeekToAudioFrames remains unchanged. It will be used to achieve sample accurate seek. 
		StreamSeekBlockIndex = INDEX_NONE;
		StreamSeekBlockOffset = INDEX_NONE;
	}
	
	// If next chunk wasn't loaded when last one finished reading, try to get it again now
	if (SrcBufferData == NULL)
	{
		UE_CLOG(WantsFilteredLogging(), LogAudioStreamCaching, Display, TEXT("[%s.%p] - Demand-loading Chunk=%d"), *WriteToString<128>(SoundWaveDataRef->GetFName()), this, CurrentChunkIndex);
		uint32 ChunkSize = 0;
		SrcBufferData = GetLoadedChunk(StreamingSoundWaveData.ToSharedRef(), CurrentChunkIndex, ChunkSize);
		if (SrcBufferData)
		{
			SrcBufferDataSize = ChunkSize;
			SrcBufferOffset = CurrentChunkIndex == 0 ? AudioDataOffset : 0;

			// If we are using a chunked seek-table, we need to parse the table first.
			if (SoundWaveDataRef->HasChunkSeekTable(CurrentChunkIndex))
			{
				ensureMsgf(FStreamedAudioChunkSeekTable::Parse(SrcBufferData, SrcBufferDataSize, SrcBufferOffset, GetCurrentSeekTable()), 
						   TEXT("Failed to parse seektable in '%s' chunk=%d"), *SoundWaveDataRef->GetFName().ToString(), CurrentChunkIndex );
			}
		}
		else
		{
			// We don't have a chunk yet. If we're just starting an on demand stream this is expected.
			const ESoundWaveLoadingBehavior Behavior = SoundWaveDataRef->GetLoadingBehavior();
			const bool bLoadOnDemandStarting = Behavior == ESoundWaveLoadingBehavior::LoadOnDemand && CurrentChunkIndex==1;
			if (!bLoadOnDemandStarting)
			{
				IStreamingManager::Get().GetAudioStreamingManager().HandleStarvation(SoundWaveDataRef, CurrentChunkIndex, OutNumBytesStreamed, BufferSize);
			}
			// Still not loaded, zero remainder of current buffer.
			ZeroBuffer(Destination, BufferSize);
			return false;
		}
	}

	bool bLooped = false;
	
	// Write out any PCM data that was decoded during the last request
	uint32 RawPCMOffset = WriteFromDecodedPCM(Destination, BufferSize);
	// immediately update the OutNumBytesStreamed to reflect what we just wrote
	OutNumBytesStreamed = RawPCMOffset;

	// if we were storing the end of the file and just now read to the end (LastPCMByteSize == 0), that means we looped
	if (bStoringEndOfFile && LastPCMByteSize == 0)
	{
		bLooped = true;
		bStoringEndOfFile = false;
	}
	
	while (RawPCMOffset < BufferSize)
	{
		if (HasError())
		{
			// If we've encountered an error, just always write zeroes and don't log anything because we already have
			// and we don't want to spam every tick.
			LastPCMByteSize = 0;
			ZeroBuffer(Destination + RawPCMOffset, BufferSize - RawPCMOffset);

			// We return true here to tell callers that we've completed decoding and should
			// be looped or otherwise terminated.
			return true;
		}

		// Decompress the next compression frame of audio (many samples) into the PCM buffer
		const int32 DecodedFrames = DecompressToPCMBuffer(/*Unused*/ 0);
		UE_CLOG(WantsFilteredLogging(), LogAudioStreamCaching, Display, TEXT("[%s.%p] - Decoded frames = %d, Chunk = %d @ %u"), *WriteToString<128>(SoundWaveDataRef->GetFName()), this, DecodedFrames, CurrentChunkIndex, SrcBufferOffset);

		if (DecodedFrames < 0)
		{
			FXxHash64 Hash = FXxHash64::HashBuffer(SrcBufferData, SrcBufferDataSize);
			UE_LOGF(LogAudioStreamCaching, Warning, "Decoder error! Zero padding and terminating... Chunk=%d, Wave=%ls DecodedFrames=%d SrcBufferOffset=%d SrcBufferDataSize=%d ChunkXxHash64=0x%llx",
				   CurrentChunkIndex, *SoundWaveDataRef->GetFName().ToString(), DecodedFrames, SrcBufferOffset, SrcBufferDataSize, Hash.Hash);

			// Flag that the decoder has an unrecoverable error, so we don't try again.
			bHasError = true;

			LastPCMByteSize = 0;
			ZeroBuffer(Destination + RawPCMOffset, BufferSize - RawPCMOffset);
			return false;
		}
		else
		{
			if (DecodedFrames > 0)
			{
				LastPCMByteSize = IncrementCurrentSampleCount(DecodedFrames * NumChannels) * MONO_PCM_SAMPLE_SIZE;

				if (StreamSeekToAudioFrames != INDEX_NONE)
				{
					// Perform frame accurate seek by discarding decoded audio.
					if (SeekToDecodedPCMFrame(StreamSeekToAudioFrames))
					{
						StreamSeekToAudioFrames = INDEX_NONE;
					}
				}

				// update OutNumBytesStreamed as we write out data
				RawPCMOffset += WriteFromDecodedPCM(Destination + RawPCMOffset, BufferSize - RawPCMOffset);
				OutNumBytesStreamed = RawPCMOffset;
			}

			const int32 PreviousChunkIndex = CurrentChunkIndex;

			// Have we reached the end of buffer?
			if (SrcBufferOffset >= SrcBufferDataSize - SrcBufferPadding)
			{
				UE_CLOG(WantsFilteredLogging(), LogAudioStreamCaching, Display, TEXT("[%s.%p] - End Of Buffer %d = %u/%u (pad = %u)"), *WriteToString<128>(SoundWaveDataRef->GetFName()), this, CurrentChunkIndex, SrcBufferOffset, SrcBufferDataSize, SrcBufferPadding);

				// Special case for the last chunk of audio
				if (CurrentChunkIndex == SoundWaveDataRef->GetNumChunks() - 1)
				{
					UE_CLOG(WantsFilteredLogging(), LogAudioStreamCaching, Display, TEXT("[%s.%p] - Last Chunk Handling LastPCMByteSize = %u"), *WriteToString<128>(SoundWaveDataRef->GetFName()), this, LastPCMByteSize);

					// check whether all decoded PCM was written
					if (LastPCMByteSize == 0)
					{
						bLooped = true;
					}
					else
					{
						bStoringEndOfFile = true;
					}

					if (bLooping)
					{
						CurrentChunkIndex = AudioDataChunkIndex;
						SrcBufferOffset = AudioDataOffset;
						CurrentSampleCount = 0;

						// Prepare the decoder to begin looping.
						PrepareToLoop();
					}
					else
					{
						// this is zero padding, so don't include it in the OutNumBytesStreamed count
						ZeroBuffer(Destination + RawPCMOffset, BufferSize - RawPCMOffset);
						break;
					}
				}
				else
				{
					CurrentChunkIndex++;
					SrcBufferOffset = 0;
				}

				SrcBufferData = GetLoadedChunk(SoundWaveDataRef, CurrentChunkIndex, SrcBufferDataSize);
				if (SrcBufferData)
				{
					// If we are using a chunked seek-table, we need to parse the table first.
					if (SoundWaveDataRef->HasChunkSeekTable(CurrentChunkIndex))
					{
						ensureMsgf(FStreamedAudioChunkSeekTable::Parse(SrcBufferData, SrcBufferDataSize, SrcBufferOffset, GetCurrentSeekTable()), 
								   TEXT("Failed to parse seektable in '%s' chunk=%d"), *SoundWaveDataRef->GetFName().ToString(), CurrentChunkIndex);
					}
					UE_CLOGF(PreviousChunkIndex != CurrentChunkIndex, LogAudio, Log, "Changed current chunk '%ls' from %d to %d, Offset %d",
							*SoundWaveDataRef->GetFName().ToString(), PreviousChunkIndex, CurrentChunkIndex, SrcBufferOffset);
				}
				else
				{
					IStreamingManager::Get().GetAudioStreamingManager().HandleStarvation(SoundWaveDataRef, CurrentChunkIndex, OutNumBytesStreamed, BufferSize);

					SrcBufferDataSize = 0;
					// this is zero padding, so don't include it in the OutNumBytesStreamed count
					ZeroBuffer(Destination + RawPCMOffset, BufferSize - RawPCMOffset);
					break;
				}
			}
		}
	}

	return bLooped;
}

int32 IStreamedCompressedInfo::DecompressToPCMBuffer(uint16 /*Unused*/ )
{
	// At the end of buffer? 
	if (SrcBufferOffset >= SrcBufferDataSize - SrcBufferPadding)
	{
		// Important we say that nothing was decoded and not an error.
		return 0;
	}

	TSharedPtr<const FSoundWaveData> SoundWaveData = GetStreamingSoundWaveData();
	if (!SoundWaveData.IsValid())
	{
		return -1;
	}
		
	// Ask for the frame size only after checking if there's space in the buffer (as the decoders will flag errors if there nothing to read).
	int32 FrameSize = GetFrameSize();
	if (FrameSize <= 0)
	{
		UE_LOGF(LogAudio, Warning, "Decoder error: Frame size negative. (indicates packet error).: Wave='%ls', FrameSize=%d, SrcBufferOffset=%u, SrcBufferDataSize=%u",
			   *SoundWaveData->GetFName().ToString(), FrameSize, SrcBufferOffset, SrcBufferDataSize);

		// Error.
		return -1;
	}
	if (SrcBufferOffset + FrameSize > SrcBufferDataSize)
	{
		UE_LOGF(LogAudio, Warning, "Decoder error: Frame size too large (decoding it will take us out of bounds).: Wave='%ls', FrameSize=%d, SrcBufferOffset=%u, SrcBufferDataSize=%u, Overby=%d", 
			   *SoundWaveData->GetFName().ToString(), FrameSize, SrcBufferOffset, SrcBufferDataSize, (int32)SrcBufferOffset+FrameSize-SrcBufferDataSize );
		
		// if frame size is too large, something has gone wrong
		return -1;
	}

	const uint8* SrcPtr = SrcBufferData + SrcBufferOffset;
	SrcBufferOffset += FrameSize;
	LastPCMOffset = 0;
	
	const FDecodeResult DecodeResult = Decode(SrcPtr, FrameSize, LastDecodedPCM.GetData(), LastDecodedPCM.Num());
	
	if (DecodeResult.NumCompressedBytesConsumed == INDEX_NONE || DECODER_MATCHES_WAVE(AudioDecompressPrivate::ForceDecoderNegativeSamplesOnWaveCVar))
	{
		UE_LOGF(LogAudio, Warning, "Decoder error: Decode call returned INDEX_NONE which indicates an error. : Wave='%ls', FrameSize=%d, SrcBufferOffset=%u, SrcBufferDataSize=%u", 
			   *SoundWaveData->GetFName().ToString(), FrameSize, SrcBufferOffset, SrcBufferDataSize);

		// Error.
		return -1;
	}

	SrcBufferOffset -= FrameSize;
	SrcBufferOffset += DecodeResult.NumCompressedBytesConsumed;
	return DecodeResult.NumAudioFramesProduced;
}

uint32 IStreamedCompressedInfo::IncrementCurrentSampleCount(uint32 NewSamples)
{
	check(CurrentSampleCount <= TrueSampleCount);
	if (CurrentSampleCount + NewSamples > TrueSampleCount)
	{
		NewSamples = TrueSampleCount - CurrentSampleCount;
		CurrentSampleCount = TrueSampleCount;
	}
	else
	{
		CurrentSampleCount += NewSamples;
	}
	return NewSamples;
}

uint32 IStreamedCompressedInfo::WriteFromDecodedPCM(uint8* Destination, uint32 BufferSize)
{
	// logical number of bytes we need to copy
	uint32 BytesToCopy = FMath::Min(BufferSize, LastPCMByteSize - LastPCMOffset);

	// make sure we aren't reading off the end of LastDecodedPCM
	BytesToCopy = FMath::Min(BytesToCopy, LastDecodedPCM.Num() - LastPCMOffset);

	if (BytesToCopy > 0)
	{
		check(BytesToCopy <= LastDecodedPCM.Num() - LastPCMOffset);
		FMemory::Memcpy(Destination, LastDecodedPCM.GetData() + LastPCMOffset, BytesToCopy);
		LastPCMOffset += BytesToCopy;
		if (LastPCMOffset >= LastPCMByteSize)
		{
			LastPCMOffset = 0;
			LastPCMByteSize = 0;
		}
	}
	return BytesToCopy;
}

bool IStreamedCompressedInfo::SeekToDecodedPCMFrame(uint32 InTargetAudioFrame)
{	
	// The MaxAvailablePCMBytePos and MinAvailablePCMBytePos are global byte positions
	// with respect to the entire sound wave. Conceptually, it is as though the entire
	// sound wave was decoded into PCM, and these values represent the portion that is
	// in the `LastDecodedPCM` buffer. In reality, because we are streaming sounds,
	// we only have the PCM data in `LastDecodedPCM` available for rendering.
	uint32 MaxAvailablePCMBytePos = CurrentSampleCount * MONO_PCM_SAMPLE_SIZE; 
	check(LastPCMByteSize <= MaxAvailablePCMBytePos);
	uint32 MinAvailablePCMBytePos = MaxAvailablePCMBytePos - LastPCMByteSize;

	uint32 TargetPCMBytePos = InTargetAudioFrame * NumChannels * MONO_PCM_SAMPLE_SIZE;

	if (TargetPCMBytePos >= MinAvailablePCMBytePos) 
	{
		LastPCMOffset = TargetPCMBytePos - MinAvailablePCMBytePos;
		if (LastPCMOffset < LastPCMByteSize)
		{
			// LastPCMOffset is set to the correct location which corresponds
			// to the target audio frame. 
			return true;
		}
		else
		{
			// More audio needs to be decoded before reaching the desired audio frame
			LastPCMOffset = 0;
			LastPCMByteSize = 0;
			return false;
		}
	}
	else
	{
		// The target audio frame is in the past. Rewinding is not supported when 
		// performing a sample accurate seek. 
		uint32 CurrentFrameOffset = MinAvailablePCMBytePos / FMath::Max(static_cast<uint32>(NumChannels * MONO_PCM_SAMPLE_SIZE), uint32(1));
		UE_LOGF(LogAudio, Warning, "Failed to seek to decoded PCM frame. Seek request for (%ls): (%ls) Chunk=%d, Offset=%d, OffsetInAudioFrames=%u, SeekInAudioFrames=%u",
			*StreamingSoundWaveData->GetFName().ToString(),
			StreamSeekToAudioFrames != INDEX_NONE ? TEXT("Using streaming seek-tables") : TEXT("Using chunk/offset pair"),
			CurrentChunkIndex,
			SrcBufferOffset,
			CurrentFrameOffset,
			StreamSeekToAudioFrames
		);

		// Return true to signal that we should no longer attempt seeks into the 
		// decoded buffer. The seek is invalid. 
		return true;
	}
}

uint32 IStreamedCompressedInfo::ZeroBuffer(uint8* Destination, uint32 BufferSize)
{
	check(Destination);

	if (BufferSize > 0)
	{
		FMemory::Memzero(Destination, BufferSize);
		return BufferSize;
	}
	return 0;
}


void IStreamedCompressedInfo::RetainNextChunkInSequence(const TSharedRef<const FSoundWaveData>& InSoundWaveData, const uint32 InChunkIndex)
{
	if (AudioDecompressPrivate::bEnableRetainingNextPlayingChunkCVar)
	{
		// Keep and hold hard ref to next chunk.
		if (const int32 NextChunk = FCachedAudioStreamingManager::GetNextChunkIndex(InSoundWaveData, InChunkIndex); NextChunk != INDEX_NONE)
		{
			NextPlayingChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(
				InSoundWaveData, NextChunk, /* bBlockForLoad*/ false, /* bForImmediatePlayback */ true);

			return;
		}
	}
	// Otherwise null it.
	NextPlayingChunkHandle = {};
}

void IStreamedCompressedInfo::RetainNextChunkInSequence(const FSoundWaveProxyPtr& InSoundWave, const uint32 InChunkIndex)
{
	if (ensure(InSoundWave.IsValid()))
	{
		RetainNextChunkInSequence(InSoundWave->GetSoundWaveDataRef(), InChunkIndex);
	}
}

const uint8* IStreamedCompressedInfo::GetLoadedChunk(const TSharedRef<const FSoundWaveData>& InSoundWaveData, uint32 ChunkIndex, uint32& OutChunkSize)
{
	if (ChunkIndex >= InSoundWaveData->GetNumChunks())
	{
		OutChunkSize = 0;
		return nullptr;
	}
	else if (ChunkIndex == 0)
	{
		UE_CLOGF(WantsFilteredLogging(), LogAudioStreamCaching, Display, "[%ls.%p] - GetLoadedChunk(0)", *WriteToString<128>(InSoundWaveData->GetFName()), this);
		TArrayView<const uint8> ZerothChunk = InSoundWaveData->GetZerothChunk(true);
		OutChunkSize = ZerothChunk.Num();
		return ZerothChunk.GetData();
	}
	else
	{
		CurCompressedChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(InSoundWaveData, ChunkIndex, false, true);
		RetainNextChunkInSequence(InSoundWaveData, ChunkIndex);
		OutChunkSize = CurCompressedChunkHandle.Num();
		UE_CLOG(WantsFilteredLogging(), LogAudioStreamCaching, Display, TEXT("[%s.%p] - GetLoadedChunk(%d, %u)"), *WriteToString<128>(InSoundWaveData->GetFName()), this, ChunkIndex, OutChunkSize);
		return CurCompressedChunkHandle.GetData();
	}
}

const uint8* IStreamedCompressedInfo::GetLoadedChunk(const FSoundWaveProxyPtr& InSoundWave, uint32 ChunkIndex, uint32& OutChunkSize)
{
	if (ensure(InSoundWave.IsValid()))
	{
		return GetLoadedChunk(InSoundWave->GetSoundWaveDataRef(), ChunkIndex, OutChunkSize);
	}
	
	OutChunkSize = 0;
	return nullptr;
}

FStreamedAudioChunkSeekTable& IStreamedCompressedInfo::GetCurrentSeekTable()
{
	if (!CurrentChunkSeekTable.IsValid())
	{
		CurrentChunkSeekTable = MakePimpl<FStreamedAudioChunkSeekTable>();
	}
	return *CurrentChunkSeekTable;
}

const FStreamedAudioChunkSeekTable& IStreamedCompressedInfo::GetCurrentSeekTable() const
{
	return const_cast<IStreamedCompressedInfo*>(this)->GetCurrentSeekTable();
}

/**
 * Worker for decompression on a separate thread
 */
FAsyncAudioDecompressWorker::FAsyncAudioDecompressWorker(USoundWave* InWave, int32 InPrecacheBufferNumFrames, FAudioDevice* InAudioDevice)
	: Wave(InWave)
	, AudioInfo(nullptr)
	, NumPrecacheFrames(InPrecacheBufferNumFrames)
{
	check(NumPrecacheFrames > 0);


	AudioInfo = IAudioInfoFactoryRegistry::Get().Create(InWave->GetRuntimeFormat());
}

void FAsyncAudioDecompressWorker::DoWork()
{
	LLM_SCOPE(ELLMTag::AudioDecompress);

	if (AudioInfo)
	{
		FSoundQualityInfo QualityInfo = { 0 };

		// Parse the audio header for the relevant information
		if (AudioInfo->ReadCompressedInfo(Wave->GetResourceData(), Wave->GetResourceSize(), &QualityInfo))
		{
			FScopeCycleCounterUObject WaveObject( Wave );

#if PLATFORM_ANDROID
			// Handle resampling
			if (QualityInfo.SampleRate > 48000)
			{
				UE_LOGF( LogAudio, Warning, "Resampling file %ls from %d", *(Wave->GetName()), QualityInfo.SampleRate);
				UE_LOGF( LogAudio, Warning, "  Size %d", QualityInfo.SampleDataSize);
				uint32 SampleCount = QualityInfo.SampleDataSize / (QualityInfo.NumChannels * sizeof(uint16));
				QualityInfo.SampleRate = QualityInfo.SampleRate / 2;
				SampleCount /= 2;
				QualityInfo.SampleDataSize = SampleCount * QualityInfo.NumChannels * sizeof(uint16);
				AudioInfo->EnableHalfRate(true);
			}
#endif

			Wave->SetSampleRate(QualityInfo.SampleRate, /* bFromDecoders*/ true);
			Wave->NumChannels = QualityInfo.NumChannels;
			if (QualityInfo.Duration > 0.0f)
			{ 
				Wave->Duration = QualityInfo.Duration;
			}

			if (Wave->DecompressionType == DTYPE_RealTime)
			{
				LLM_SCOPE(ELLMTag::AudioRealtimePrecache);
#if PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS > 0
				const uint32 PCMBufferSize = NumPrecacheFrames * MONO_PCM_SAMPLE_SIZE * Wave->NumChannels * PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS;
				Wave->NumPrecacheFrames = NumPrecacheFrames;
				if (Wave->CachedRealtimeFirstBuffer == nullptr)
				{
					Wave->CachedRealtimeFirstBuffer = (uint8*)FMemory::Malloc(PCMBufferSize);
					AudioInfo->ReadCompressedData(Wave->CachedRealtimeFirstBuffer, Wave->bLooping, PCMBufferSize);
				}
				else if (Wave->GetPrecacheState() == ESoundWavePrecacheState::Done)
				{
					UE_LOGF(LogAudio, Warning, "Attempted to precache decoded audio multiple times.");
				}
#endif
			}
			else
			{
				LLM_SCOPE(ELLMTag::AudioFullDecompress);

				Wave->RawPCMDataSize = QualityInfo.SampleDataSize;
				check(Wave->RawPCMData == nullptr);
				Wave->RawPCMData = ( uint8* )FMemory::Malloc( Wave->RawPCMDataSize );

				// Decompress all the sample data into preallocated memory
				AudioInfo->ExpandFile(Wave->RawPCMData, &QualityInfo);

				// Only track the RawPCMDataSize at this point since we haven't yet
				// removed the compressed asset from memory and GetResourceSize() would add that at this point
				Wave->TrackedMemoryUsage += Wave->RawPCMDataSize;
				INC_DWORD_STAT_BY(STAT_AudioMemorySize, Wave->RawPCMDataSize);
				INC_DWORD_STAT_BY(STAT_AudioMemory, Wave->RawPCMDataSize);
			}
		}
		else if (Wave->DecompressionType == DTYPE_RealTime)
		{
			Wave->DecompressionType = DTYPE_Invalid;
			Wave->NumChannels = 0;

			Wave->RemoveAudioResource();
		}

		delete AudioInfo;
	}
}

static TAutoConsoleVariable<int32> CVarShouldUseBackgroundPoolFor_FAsyncRealtimeAudioTask(
	TEXT("AudioThread.UseBackgroundThreadPool"),
	1,
	TEXT("If true, use the background thread pool for realtime audio decompression."));

bool ShouldUseBackgroundPoolFor_FAsyncRealtimeAudioTask()
{
	return !!CVarShouldUseBackgroundPoolFor_FAsyncRealtimeAudioTask.GetValueOnAnyThread();
}

// end

bool ICompressedAudioInfo::StreamCompressedInfo(USoundWave* Wave, FSoundQualityInfo* QualityInfo)
{
	if (!Wave)
	{
		return false;
	}

	// Create and cache proxy object
	TSharedRef<const FSoundWaveProxy> Proxy = Wave->GetSoundWaveProxy();
	
	StreamingSoundWaveData = Proxy->GetSoundWaveDataRef();
	check(StreamingSoundWaveData.IsValid());

#if !UE_BUILD_SHIPPING
	bWantsFilteredLogging = Audio::MatchesLogFilter(*GetNameSafe(Wave));
#endif //UE_BUILD_SHIPPING
	
	return StreamCompressedInfoInternal(StreamingSoundWaveData.ToSharedRef(), QualityInfo);
}

bool ICompressedAudioInfo::StreamCompressedInfo(const FSoundWaveProxyPtr& Wave, FSoundQualityInfo* QualityInfo)
{
	if (ensure(Wave.IsValid()))
	{
		return StreamCompressedInfo(Wave->GetSoundWaveDataRef(), QualityInfo);
	}
	
	return false;
}

bool ICompressedAudioInfo::StreamCompressedInfo(const TSharedRef<const FSoundWaveData>& Wave, struct FSoundQualityInfo* QualityInfo)
{
	// Create our own copy of the proxy object
	StreamingSoundWaveData = Wave;

#if !UE_BUILD_SHIPPING
	bWantsFilteredLogging = Audio::MatchesLogFilter(Wave->GetFName());
#endif //UE_BUILD_SHIPPING

	return StreamCompressedInfoInternal(StreamingSoundWaveData.ToSharedRef(), QualityInfo);
}

IAudioInfoFactoryRegistry& IAudioInfoFactoryRegistry::Get()
{
	static struct FConcreteRegistry : IAudioInfoFactoryRegistry
	{
		mutable FRWLock FactoriesRWLock;
		TMap<FName, IAudioInfoFactory*> Factories;
		TArray<IAudioDecoderOverride*> Overrides;

		void Register(IAudioInfoFactory* InFactory, FName InFormatName) override
		{
			FRWScopeLock Lock(FactoriesRWLock, FRWScopeLockType::SLT_Write);
			UE_LOGF(LogAudio, Display, "AudioInfo: '%ls' Registered", *InFormatName.ToString());
			check(!Factories.Contains(InFormatName));
			Factories.Add(InFormatName) = InFactory;
		}
		void Unregister(IAudioInfoFactory* InFactory, FName InFormatName) override
		{
			FRWScopeLock Lock(FactoriesRWLock, FRWScopeLockType::SLT_Write);

			Factories.Remove(InFormatName);
		}
		IAudioInfoFactory* Find(FName InFormat) const override
		{
			FRWScopeLock Lock(FactoriesRWLock, FRWScopeLockType::SLT_ReadOnly);
			if( IAudioInfoFactory* const* Factory = Factories.Find(InFormat))
			{
				return *Factory;
			}
			return nullptr;
		}

		ICompressedAudioInfo* Create(FName InFormat, FName InAssetName) const override
		{
			if (!Overrides.IsEmpty() && !InAssetName.IsNone())
			{
				// Check if we want to create an override decoder for this asset
				FRWScopeLock Lock(FactoriesRWLock, FRWScopeLockType::SLT_ReadOnly);
				for (IAudioDecoderOverride* Override : Overrides)
				{
					ICompressedAudioInfo* Info = Override->CreateOverrideDecoder(InAssetName);
					if (Info)
					{
						return Info;
					}
				}
			}
			if (IAudioInfoFactory* Factory = Find(InFormat))
			{
				return Factory->Create();
			}
			return nullptr;
		}

		virtual void RegisterOverride(IAudioDecoderOverride* Override) override
		{
			FRWScopeLock Lock(FactoriesRWLock, FRWScopeLockType::SLT_Write);
			Overrides.Add(Override);
		}

		virtual void UnregisterOverride(IAudioDecoderOverride* Override) override
		{
			FRWScopeLock Lock(FactoriesRWLock, FRWScopeLockType::SLT_Write);
			Overrides.Remove(Override);
		}

	} sInstance;
	return sInstance;
}
