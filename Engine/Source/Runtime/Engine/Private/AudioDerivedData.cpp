// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioDerivedData.h"
#include "AudioThread.h"
#include "Interfaces/IAudioFormat.h"
#include "Misc/CommandLine.h"
#include "Async/AsyncWork.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/ScopedSlowTask.h"
#include "Audio.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Async/Async.h"
#include "SoundWaveCompiler.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheKey.h"
#include "ProfilingDebugging/CookStats.h"
#include "AudioResampler.h"
#include "AudioCompressionSettingsUtils.h"
#include "Sound/SoundSourceBus.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProcedural.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/MultichannelBuffer.h"
#include "Containers/UnrealString.h"
#include "Sound/StreamedAudioChunkSeekTable.h"
#include "AudioDecompress.h"
#include "ISoundWaveCloudStreaming.h"
#include "Serialization/MemoryReader.h"
#include "IO/IoHash.h"

#if WITH_EDITORONLY_DATA
#include "Serialization/MemoryWriter.h"
#endif

#if WITH_EDITOR
#include "UObject/ArchiveCookContext.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioDerivedData)

DEFINE_LOG_CATEGORY_STATIC(LogAudioDerivedData, Log, All);

static int32 AllowAsyncCompression = 1;
FAutoConsoleVariableRef CVarAllowAsyncCompression(
	TEXT("au.compression.AsyncCompression"),
	AllowAsyncCompression,
	TEXT("1: Allow async compression of USoundWave when supported by the codec.\n")
	TEXT("0: Disable async compression."),
	ECVF_Default);

#define FORCE_RESAMPLE 0

#if ENABLE_COOK_STATS
namespace AudioCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStats::FDDCResourceUsageStats StreamingChunkUsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("Audio.Usage"), TEXT("Inline"));
		StreamingChunkUsageStats.LogStats(AddStat, TEXT("Audio.Usage"), TEXT("Streaming"));
	});
}
#endif

#if WITH_EDITORONLY_DATA

namespace AudioDerivedDataPrivate
{
	constexpr float FloatToPcm16Scalar = 32767.f;

	// This function for converting pcm16 to float is used so that existing 
	// assets cook to the exact same bit-wise result. While it would be nice to 
	// replace the divide operator with a multiply operator in the for loop, this 
	// should not be done as it produces different results.
	void ArrayPcm16ToFloat(TArrayView<const int16> InView, TArrayView<float> OutView)
	{
		check(InView.Num() == OutView.Num());

		const int16* InData = InView.GetData();
		float* OutData = OutView.GetData();
		const uint32 Num = InView.Num();

		for (uint32 i = 0; i < Num; i++)
		{
			OutData[i] = (float)InData[i] / FloatToPcm16Scalar;
		}
	}

	void ArrayFloatToPcm16(TArrayView<const float> InView, TArrayView<int16> OutView)
	{
		check(InView.Num() == OutView.Num());

		const float* InData = InView.GetData();
		int16* OutData = OutView.GetData();
		const uint32 Num = InView.Num();

		for (uint32 i = 0; i < Num; i++)
		{
			OutData[i] = (int16)(InData[i] * FloatToPcm16Scalar);
		}
	}
}

// Any thread implicated in the streamed audio platform data build must have a valid scope to be granted access
// to properties being modified by the build itself without triggering a FinishCache.
// Any other thread that is a consumer of the FStreamedAudioPlatformData will trigger a FinishCache when accessing
// incomplete properties which will wait until the builder thread has finished before returning property that is ready to be read.
class FStreamedAudioBuildScope
{
public:
	FStreamedAudioBuildScope(const FStreamedAudioPlatformData* PlatformData)
	{
		PreviousScope = PlatformDataBeingAsyncCompiled;
		PlatformDataBeingAsyncCompiled = PlatformData;
	}

	~FStreamedAudioBuildScope()
	{
		check(PlatformDataBeingAsyncCompiled);
		PlatformDataBeingAsyncCompiled = PreviousScope;
	}

	static bool ShouldWaitOnIncompleteProperties(const FStreamedAudioPlatformData* PlatformData)
	{
		return PlatformDataBeingAsyncCompiled != PlatformData;
	}

private:
	const FStreamedAudioPlatformData* PreviousScope = nullptr;
	// Only the thread(s) compiling this platform data will have full access to incomplete properties without causing any stalls.
	static thread_local const FStreamedAudioPlatformData* PlatformDataBeingAsyncCompiled;
};

thread_local const FStreamedAudioPlatformData* FStreamedAudioBuildScope::PlatformDataBeingAsyncCompiled = nullptr;

#endif  // #ifdef WITH_EDITORONLY_DATA

TIndirectArray<struct FStreamedAudioChunk>& FStreamedAudioPlatformData::GetChunks() const
{
#if WITH_EDITORONLY_DATA
	if (FStreamedAudioBuildScope::ShouldWaitOnIncompleteProperties(this))
	{
		// For the chunks to be available, any async task need to complete first.
		const_cast<FStreamedAudioPlatformData*>(this)->FinishCache();
	}
#endif
	return const_cast<FStreamedAudioPlatformData*>(this)->Chunks;
}

int32 FStreamedAudioPlatformData::GetNumChunks() const
{
#if WITH_EDITORONLY_DATA
	if (FStreamedAudioBuildScope::ShouldWaitOnIncompleteProperties(this))
	{
		// NumChunks is written by the caching process, any async task need to complete before we can read it.
		const_cast<FStreamedAudioPlatformData*>(this)->FinishCache();
	}
#endif

	return Chunks.Num();
}

FName FStreamedAudioPlatformData::GetAudioFormat() const
{
#if WITH_EDITORONLY_DATA
	if (FStreamedAudioBuildScope::ShouldWaitOnIncompleteProperties(this))
	{
		// AudioFormat is written by the caching process, any async task need to complete before we can read it.
		const_cast<FStreamedAudioPlatformData*>(this)->FinishCache();
	}
#endif

	return AudioFormat;
}


/*------------------------------------------------------------------------------
Derived data key generation.
------------------------------------------------------------------------------*/

// If you want to bump this version, generate a new guid using
// VS->Tools->Create GUID and paste it here. https://www.guidgen.com works too.
#define AUDIO_DERIVEDDATA_VER				TEXT("64e45415311549acbee28698853aa3a4")				// Change this if you want to bump all audio formats at once
#define STREAMEDAUDIO_DERIVEDDATA_VER		TEXT("479371f6537b4fc1be91786813e83a16")				// This depends on the above key, but will regenerate all streaming chunk data derived from the compressed audio

#if WITH_EDITORONLY_DATA

#ifndef CASE_ENUM_TO_TEXT
#define CASE_ENUM_TO_TEXT(X) case X: return TEXT(#X);
#endif

const TCHAR* LexToString(const ESoundwaveSampleRateSettings Enum)
{
	switch (Enum)
	{
		FOREACH_ENUM_ESOUNDWAVESAMPLERATESETTINGS(CASE_ENUM_TO_TEXT)
	}
	return TEXT("<Unknown ESoundwaveSampleRateSettings>");
}

static FString GetSoundWaveHash(const USoundWave& InWave, const ITargetPlatform* InTargetPlatform)
{
	// Hash the parts of the SoundWave that can affect the compressed data. It doesn't hurt to do this. 
	// Typically the GUID will change if compressed data changes, but some settings can affect 
	// the SoundWave. i.e. DefaultSoundWaveQuality which won't change the GUID. So is better
	// it's reflected here.
	using FPCU = FPlatformCompressionUtilities;
	FString SoundWaveHash;
	FPCU::AppendHash(SoundWaveHash, TEXT("QLT"), InWave.GetCompressionQuality());
	FPCU::AppendHash(SoundWaveHash, TEXT("CHN"), InWave.NumChannels);
	FPCU::AppendHash(SoundWaveHash, TEXT("SRQ"), InWave.SampleRateQuality);
	FPCU::AppendHash(SoundWaveHash, TEXT("CK1"), InWave.GetSizeOfFirstAudioChunkInSeconds(InTargetPlatform));

	// Add cloud streaming parameters, if available and enabled.
	if (InWave.IsCloudStreamingEnabled())
	{
		IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
		TArray<Audio::ISoundWaveCloudStreamingFeature*> Features = IModularFeatures::Get().GetModularFeatureImplementations<Audio::ISoundWaveCloudStreamingFeature>(Audio::ISoundWaveCloudStreamingFeature::GetModularFeatureName());
		for(int32 i=0; i<Features.Num(); ++i)
		{
			if (Features[i]->CanOverrideFormat(&InWave))
			{
				FString Hash = Features[i]->GetOverrideParameterDDCHash(&InWave);
				FPCU::AppendHash(SoundWaveHash, TEXT("CSP"), Hash);
				break;
			}
		}
	}

	return SoundWaveHash;
}

static FString GetSoundWaveRawDataHash(const USoundWave& InWave)
{
	const USoundWave::FEditorAudioBulkData& RawData = InWave.RawData;
	return LexToString(RawData.RawData.GetPayloadId());
}

static FString GetWaveformTransformationHash(const USoundWave& InWave)
{
	// Hash the SoundWave waveform transformations 
	FString WaveformTransformationHash;

	const TArray<TObjectPtr<UWaveformTransformationBase>>& Transformations = InWave.Transformations;
	
	for (const TObjectPtr<UWaveformTransformationBase>& Transformation : Transformations)
	{
		if (Transformation)
		{
			WaveformTransformationHash += Transformation->GetTransformationHash();
		}
	}

	return WaveformTransformationHash;
}

/**
 * Computes the derived data key suffix for a SoundWave's Streamed Audio.
 * @param SoundWave - The SoundWave for which to compute the derived data key.
 * @param AudioFormatName - The audio format we're creating the key for
 * @param OutKeySuffix - The derived data key suffix.
 */
static void GetStreamedAudioDerivedDataKeySuffix(
	const USoundWave& SoundWave,
	FName AudioFormatName,
	const FPlatformAudioCookOverrides* CompressionOverrides,
	const ITargetPlatform* InTargetPlatform,
	FString& OutKeySuffix
	)
{
	uint16 Version = 0;
	bool bFormatUsingStreamingSeekTables = false;

	// get the version for this soundwave's platform format
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (TPM)
	{
		const IAudioFormat* AudioFormat = TPM->FindAudioFormat(AudioFormatName);
		if (AudioFormat)
		{
			Version = AudioFormat->GetVersion(AudioFormatName);
			bFormatUsingStreamingSeekTables = AudioFormat->RequiresStreamingSeekTable();
		}
	}

	FString AudioFormatNameString = AudioFormatName.ToString();

	// If we have compression overrides for this target platform, append them to this string.
	if (CompressionOverrides)
	{
		FPlatformAudioCookOverrides::GetHashSuffix(CompressionOverrides, AudioFormatNameString);
	}

	// If format uses streaming seek-tables append the version into the hash.
	// Initial version '0' won't change the key.
	if (const int16 SeekTableVersion = FStreamedAudioChunkSeekTable::GetVersion(); SeekTableVersion != 0 && bFormatUsingStreamingSeekTables)
	{
		AudioFormatNameString += FString::Printf(TEXT("_STVER=%d"), SeekTableVersion);
	}

	const FString SoundWaveHash = GetSoundWaveHash(SoundWave, InTargetPlatform);
		
	// build the key
	OutKeySuffix = FString::Printf(TEXT("%s_%d_%s_%s"),
		*AudioFormatNameString,
		Version,
		*SoundWaveHash,
		*SoundWave.CompressedDataGuid.ToString()
	);

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	OutKeySuffix.Append(TEXT("_arm64"));
#endif
}

/**
 * Constructs a derived data key from the key suffix.
 * @param KeySuffix - The key suffix.
 * @param OutKey - The full derived data key.
 */
static void GetStreamedAudioDerivedDataKeyFromSuffix(const FString& KeySuffix, FString& OutKey)
{
	OutKey = FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("STREAMEDAUDIO"),
		STREAMEDAUDIO_DERIVEDDATA_VER,
		*KeySuffix
		);
}

/**
 * Constructs the derived data key for an individual audio chunk.
 * @param KeySuffix - The key suffix.
 * @param ChunkIndex - The chunk index.
 * @param OutKey - The full derived data key for the audio chunk.
 */
static void GetStreamedAudioDerivedChunkKey(
	int32 ChunkIndex,
	const FStreamedAudioChunk& Chunk,
	const FString& KeySuffix,
	FString& OutKey
	)
{
	OutKey = FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("STREAMEDAUDIO"),
		STREAMEDAUDIO_DERIVEDDATA_VER,
		*FString::Printf(TEXT("%s_CHUNK%u_%d"), *KeySuffix, ChunkIndex, Chunk.DataSize)
		);
}

/**
 * Computes the derived data key for Streamed Audio.
 * @param SoundWave - The soundwave for which to compute the derived data key.
 * @param AudioFormatName - The audio format we're creating the key for
 * @param OutKey - The derived data key.
 */
static void GetStreamedAudioDerivedDataKey(
	const USoundWave& SoundWave,
	FName AudioFormatName,
	const FPlatformAudioCookOverrides* CompressionOverrides,
	const ITargetPlatform* InTargetPlatform,
	FString& OutKey
	)
{
	FString KeySuffix;
	GetStreamedAudioDerivedDataKeySuffix(SoundWave, AudioFormatName, CompressionOverrides, InTargetPlatform, KeySuffix);
	GetStreamedAudioDerivedDataKeyFromSuffix(KeySuffix, OutKey);
}

/**
 * Gets Wave format for a SoundWave on the current running platform
 * @param SoundWave - The SoundWave to get format for.
 */
static FName GetWaveFormatForRunningPlatform(USoundWave& SoundWave)
{
	// Compress to whatever format the active target platform wants
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (TPM)
	{
		ITargetPlatform* CurrentPlatform = TPM->GetRunningTargetPlatform();

		check( CurrentPlatform != nullptr );

		return CurrentPlatform->GetWaveFormat(&SoundWave);
	}

	return NAME_None;
}

static const FPlatformAudioCookOverrides* GetCookOverridesForRunningPlatform()
{
	return FPlatformCompressionUtilities::GetCookOverrides(nullptr);
}

/**
 * Stores derived data in the DDC.
 * After this returns, all bulk data from streaming chunks will be sent separately to the DDC and the BulkData for those chunks removed.
 * @param DerivedData - The data to store in the DDC.
 * @param DerivedDataKeySuffix - The key suffix at which to store derived data.
 * @return number of bytes put to the DDC (total, including all chunks)
 */
static uint32 PutDerivedDataInCache(
	FStreamedAudioPlatformData* DerivedData,
	const FString& DerivedDataKeySuffix,
	const FStringView& SoundWaveName
	)
{
	TArray<uint8> RawDerivedData;
	FString DerivedDataKey;
	uint32 TotalBytesPut = 0;

	// Build the key with which to cache derived data.
	GetStreamedAudioDerivedDataKeyFromSuffix(DerivedDataKeySuffix, DerivedDataKey);

	FString LogString;
	if (UE_LOG_ACTIVE(LogAudio,Verbose))
	{
		LogString = FString::Printf(
			TEXT("Storing Streamed Audio in DDC:\n  Key: %s\n  Format: %s\n"),
			*DerivedDataKey,
			*DerivedData->AudioFormat.ToString()
			);
	}

	// Write out individual chunks to the derived data cache.
	const int32 ChunkCount = DerivedData->Chunks.Num();
	for (int32 ChunkIndex = 0; ChunkIndex < ChunkCount; ++ChunkIndex)
	{
		FString ChunkDerivedDataKey;
		FStreamedAudioChunk& Chunk = DerivedData->Chunks[ChunkIndex];
		GetStreamedAudioDerivedChunkKey(ChunkIndex, Chunk, DerivedDataKeySuffix, ChunkDerivedDataKey);

		if (UE_LOG_ACTIVE(LogAudio,Verbose))
		{
			LogString += FString::Printf(TEXT("  Chunk%d %" INT64_FMT " bytes %s\n"),
				ChunkIndex,
				Chunk.BulkData.GetBulkDataSize(),
				*ChunkDerivedDataKey
				);
		}

		TotalBytesPut += Chunk.StoreInDerivedDataCache(ChunkDerivedDataKey, SoundWaveName);
	}

	// Store derived data.
	// At this point we've stored all the non-inline data in the DDC, so this will only serialize and store the metadata and any inline chunks
	FMemoryWriter Ar(RawDerivedData, /*bIsPersistent=*/ true);
	DerivedData->Serialize(Ar, NULL);
	GetDerivedDataCacheRef().Put(*DerivedDataKey, RawDerivedData, SoundWaveName);
	TotalBytesPut += RawDerivedData.Num();
	UE_LOGF(LogAudio, Verbose, "%ls  Derived Data: %d bytes", *LogString, RawDerivedData.Num());
	return TotalBytesPut;
}

namespace EStreamedAudioCacheFlags
{
	enum Type
	{
		None			= 0x0,
		Async			= 0x1,
		ForceRebuild	= 0x2,
		InlineChunks	= 0x4,
		AllowAsyncBuild	= 0x8,
		ForDDCBuild		= 0x10,
	};
};

/**
 * Worker used to cache streamed audio derived data.
 */
class FStreamedAudioCacheDerivedDataWorker : public FNonAbandonableTask
{
	/** Where to store derived data. */
	FStreamedAudioPlatformData* DerivedData = nullptr;
	/** Path of the SoundWave being cached for logging purpose. */
	FString SoundWavePath;
	/** Full name of the SoundWave being cached for logging purpose. */
	FString SoundWaveFullName;
	/** Audio Format Name */
	FName AudioFormatName;
	/** Derived data key suffix. */
	FString KeySuffix;
	/** Streamed Audio cache flags. */
	uint32 CacheFlags = 0;
	/** Have many bytes were loaded from DDC or built (for telemetry) */
	uint32 BytesCached = 0;
	/** Sample rate override specified for this sound wave. */
	const FPlatformAudioCookOverrides* CompressionOverrides = nullptr;
	/** true if caching has succeeded. */
	bool bSucceeded = false;
	/** true if the derived data was pulled from DDC */
	bool bLoadedFromDDC = false;
	/** Already tried to built once */
	bool bHasBeenBuilt = false;
	/** Handle for retrieving compressed audio for chunking */
	uint32 CompressedAudioHandle = 0;
	/** If the wave file is procedural */
	bool bIsProcedural = false;
	/** If the wave file is streaming */
	bool bIsStreaming = false;
	/** Initial chunk size */
	int32 ZerothChunkSizeSoundWaveOverride = 0;
	/* Size in Seconds of First Audio Chunk */
	float SizeOfFirstAudioChunkInSeconds = 0.f;

	bool GetCompressedData(FAudioCookOutputs& OutData)
	{
		if (CompressedAudioHandle)
		{
			GetDerivedDataCacheRef().WaitAsynchronousCompletion(CompressedAudioHandle);
			TArray<uint8> ByteStream;
			const bool bResult = GetDerivedDataCacheRef().GetAsynchronousResults(CompressedAudioHandle, ByteStream);
			if (bResult)
			{
				FMemoryReader Reader(ByteStream);
				ensure(OutData.Serialize(Reader));
			}
			CompressedAudioHandle = 0;
			return bResult && OutData.EncodedData.Num() > 0;
		}

		return false;
	}

	static void MakeChunkSeekTable(const IAudioFormat::FSeekTable& InTable, uint32 InChunkStart, uint32 InChunkEnd, FStreamedAudioChunkSeekTable& Out)
	{
		int32 SearchFrom = Algo::LowerBound(InTable.Offsets, InChunkStart);
		if (SearchFrom < 0)
		{
			return;
		}

		for (int32 i=SearchFrom;i<InTable.Offsets.Num();++i)
		{
			uint32 Offset = InTable.Offsets[i];
			uint32 TimeInAudioFrames = InTable.Times[i];

			if (Offset > InChunkEnd) 
			{
				break;
			}
			if (Offset >= InChunkStart && Offset < InChunkEnd)
			{
				Out.Add(TimeInAudioFrames,Offset - InChunkStart);
			}
		}
	}	

	// Returns the size of the new seek-table in bytes.
	static int32 PrefixChunkWithSeekTable(FStreamedAudioChunkSeekTable& InSeekTable, TArray<uint8>& InOutChunkBytes)
	{	
		TArray<uint8> Bytes;
		FMemoryWriter Writer(Bytes);
		InSeekTable.Serialize(Writer);
		InOutChunkBytes.Insert(Bytes,0);
		return Bytes.Num();
	}

	static EChunkSeekTableMode DetermineSeekTableMode(const IAudioFormat::FSeekTable& InTable)
	{
		const TArray<uint32>& Times = InTable.Times;
		if (InTable.Times.Num()>2)
		{
			int32 Delta = Times[1] - Times[0];
			for (int32 i = 2; i < Times.Num(); ++i)
			{
				if (Times[i] - Times[i - 1] != Delta)
				{
					return EChunkSeekTableMode::VariableSamplesPerEntry;
				}
			}
		}
		return EChunkSeekTableMode::ConstantSamplesPerEntry;
	}

	static void CreateChunkSeektableAndPrefix(const EChunkSeekTableMode InMode, const IAudioFormat::FSeekTable& InTable, const uint32 InChunkStart, const uint32 InChunkEnd, 
		TArray<uint8>& OutChunk, TArray<uint32>& OutChunkOffsets, const int32 InEstimatedSeekTableSize, const int32 InBlockCount, const uint32 InAudioSize, const int32 InBudget)
	{
		FStreamedAudioChunkSeekTable ChunkTable(InMode);
		MakeChunkSeekTable(InTable, InChunkStart, InChunkEnd, ChunkTable);
		const int32 SeekTableSize = PrefixChunkWithSeekTable(ChunkTable, OutChunk);

		ensureMsgf(SeekTableSize == InEstimatedSeekTableSize, TEXT("FStreamedAudioChunkSeekTable:CalcSize doesn't match Serialize. SerializedSize=%d, CaclSize=%d"), SeekTableSize, InEstimatedSeekTableSize);
		ensureMsgf(OutChunk.Num() <= InBudget, TEXT("Chunk is over budget: Size=%d, Budget=%d"), OutChunk.Num(), InBudget);
		ensure(SeekTableSize + InAudioSize == OutChunk.Num());
		ensureMsgf(InBlockCount == ChunkTable.Num(), TEXT("Expecting to have same number of items in our table as we're budgeted for: Planned=%d, Table=%d"), InBlockCount, ChunkTable.Num());
	
		OutChunkOffsets.Add(ChunkTable.FindTime(0));
	}

	bool SplitUsingSeekTable(const IAudioFormat::FSeekTable& InTable, const FAudioCookOutputs& InCookOuputs,
	                         TArray<TArray<uint8>>& OutBuffers,
	                         const int32 InFirstChunkMaxSize,const int32 InMaxChunkSize,
	                         TArray<uint32>& OutChunkOffsets,
	                         const uint32 InFirstAudioChunkMaxLengthInAudioFrames,
	                        const uint32 InMaxChunkLengthInFrames) const
	{
		// Typical layout:
		// Chunk 0 - Header (always inlined)
		// Chunk 1 - Audio (optionally inlined, containing n secs of audio).
		// Chunk N - Audio - Sized using Max Chunk size.
		
		const TArray<uint8>& InSrcBuffer = InCookOuputs.EncodedData;
		const TArray<uint32>& Offsets = InTable.Offsets;
		const TArray<uint32>& Times = InTable.Times;

		// Table needs to have equal number of time entries as offsets.
		if (!ensure(Offsets.Num() == Times.Num()))
		{
			return false;
		}
		 
		// Reject bad inputs.
		if (Offsets.IsEmpty() || InSrcBuffer.IsEmpty() || InFirstChunkMaxSize <= 0 || InMaxChunkSize <=0)
		{
			return false;
		}

		const EChunkSeekTableMode Mode = DetermineSeekTableMode(InTable);

		uint8 const* Source = InSrcBuffer.GetData();
		const uint32 SourceLen = InSrcBuffer.Num();
		uint8 const* SourceEnd = Source + SourceLen;

		uint8 const* ChunkStart = Source;
		uint8 const* Current = Source + Offsets[0];	

		int32 BlockCount = 0;
		int32 CurrentBlock = 0;
		int32 CurrentAudioFrameCount = 0;

		uint32 Budget = InFirstChunkMaxSize;
		uint32 AudioFrameBudget = InFirstAudioChunkMaxLengthInAudioFrames;

		while (Current < SourceEnd)
		{	
			// A block is the span between two neighboring offsets in our table.
			const uint32 BlockStart = Offsets[CurrentBlock];
			const uint32 BlockEnd = CurrentBlock < Offsets.Num() - 1 ? Offsets[CurrentBlock + 1] : SourceLen;
			if (!ensureMsgf(BlockStart < BlockEnd, TEXT("Malformed Table: BlockStart=%u, BlockEnd=%u"), BlockStart, BlockEnd))
			{
				return false;
			}
			const uint32 BlockSize = BlockEnd - BlockStart;
			if (!ensureMsgf(BlockSize < 1024*1024, TEXT("Malformed Table: Block is too large to be real. BlockSize=%u"), BlockSize) )
			{
				return false;
			}

			// Tally the duration of this block.
			// Note we don't know the size of the very last block, so we set it 0. For this purpose that's fine.
			const uint32 BlockStartTime = Times[CurrentBlock];
			const uint32 BlockEndTime = CurrentBlock + 1 < Times.Num()-1 ? Times[CurrentBlock + 1] : BlockStartTime;
			if (!ensureMsgf(BlockStartTime <= BlockEndTime, TEXT("Malformed Table: Block times out of order. BlockStartTime=%u, BlockEndTime=%u, CurrentBlock=%d"), BlockStartTime, BlockEnd,  CurrentBlock))
			{
				return false;
			}
			const uint32 BlockLengthAudioFrames = BlockEndTime - BlockStartTime;
						
			// Current chunk stats.
			const int32 CurrentTableSize = FStreamedAudioChunkSeekTable::CalcSize(BlockCount, Mode);
			checkf(CurrentTableSize >= 0, TEXT("CurrentTableSize=%d"), CurrentTableSize);
			const uint32 CurrentChunkSize = (Current - ChunkStart) + CurrentTableSize;
			
			// Adding a new item will grow the table by 1.
			const int32 NewTableSize = FStreamedAudioChunkSeekTable::CalcSize(BlockCount + 1, Mode);
			checkf(NewTableSize >= 0, TEXT("NewTableSize=%d"), NewTableSize);
			const int32 TableDelta = NewTableSize - CurrentTableSize;
			checkf(TableDelta > 0, TEXT("Table should grow when we add a new entry: TableDelta=%d"), TableDelta);
			
			// Accumulated enough? (bytes or audio frames).
			check(Budget > 0);
			const bool bOverSize = (CurrentChunkSize + BlockSize + TableDelta) > Budget;
			const bool bOverLength = AudioFrameBudget > 0 && CurrentAudioFrameCount > 0 && (CurrentAudioFrameCount + BlockLengthAudioFrames > AudioFrameBudget);
			
			if (bOverSize || bOverLength)
			{
				// can't add this chunk, emit.
				TArray<uint8> Chunk(ChunkStart, Current - ChunkStart);
				const int32 AudioSize = Chunk.Num();

				CreateChunkSeektableAndPrefix(Mode, InTable, ChunkStart-Source, Current-Source, Chunk, OutChunkOffsets, CurrentTableSize, BlockCount, AudioSize, Budget);
		
				UE_LOGF(LogAudio, Verbose, "Adding Chunk %d because (%ls): Blocks=%d (%u bytes), AudioFrames=%u (%2.2f seconds), SeekTableEntries=%d (%d bytes), ChunkSize=%u bytes, PercentFull=%2.2f, Remaining=%u bytes",
					OutBuffers.Num(), bOverSize ? TEXT("Enough Bytes") : TEXT("Enough Samples"), BlockCount, AudioSize, CurrentAudioFrameCount,
					(float)CurrentAudioFrameCount / InCookOuputs.SampleRate, BlockCount, CurrentTableSize, Chunk.Num(), ((float)Chunk.Num() / Budget) * 100.f, Budget-Chunk.Num()
					);

				OutBuffers.Add(Chunk);

				// Reset our pointers for a new chunk.
				ChunkStart = Current;
				BlockCount = 0;
				CurrentAudioFrameCount = 0;
				
				// Reset size budget. (note different budget after chunk 0).
				Budget = InMaxChunkSize;

				// Reset length budget. (note different budget for chunk 1, the audio chunk).
				// The first "Audio" chunk is considered to be Chunk 1				
				const bool bIsFirstAudioChunk = OutBuffers.Num() == 1;
				AudioFrameBudget = bIsFirstAudioChunk ? InFirstAudioChunkMaxLengthInAudioFrames : InMaxChunkLengthInFrames;
				
				// retry.
				continue;
			}
				
			// Include this block.
			Current += BlockSize;
			CurrentAudioFrameCount += BlockLengthAudioFrames;
			BlockCount++;
			CurrentBlock++;

			UE_LOGF(LogAudio, VeryVerbose, "\tAdding Block: Chunk_Block_Count=%d, BlockSize=%d, ChunkSize=%lld", BlockCount, BlockSize, Current - ChunkStart);
		}

		// emit any remainder chunks
		if (Current - ChunkStart)
		{
			// emit this chunk
			TArray<uint8> Chunk(ChunkStart, Current - ChunkStart);
			const int32 AudioSize = Chunk.Num();
			const int32 CurrentTableSize = FStreamedAudioChunkSeekTable::CalcSize(BlockCount, Mode);
			CreateChunkSeektableAndPrefix(Mode, InTable, ChunkStart - Source, Current - Source, Chunk, OutChunkOffsets, CurrentTableSize, BlockCount, AudioSize, Budget);

			UE_LOGF(LogAudio, Verbose, "Adding FINAL Chunk %d because (%ls): Blocks=%d (%u bytes), AudioFrames=%u (%2.2f seconds), SeekTableEntries=%d (%d bytes), ChunkSize=%u bytes, PercentFull=%2.2f, Remaining=%u bytes",
				OutBuffers.Num(), TEXT("EOF"), BlockCount, AudioSize, CurrentAudioFrameCount,
				(float)CurrentAudioFrameCount / InCookOuputs.SampleRate, BlockCount, CurrentTableSize, Chunk.Num(), ((float)Chunk.Num() / Budget) * 100.f, Budget-Chunk.Num()
				);

			OutBuffers.Add(Chunk);
		}

		return true;
	}

	bool SplitDataForStreaming(const IAudioFormat* InFormat, const FAudioCookOutputs& InCookOutputs, TArray<TArray<uint8>>& OutBuffers, const int32 InFirstChunkMaxSize, const int32 InMaxChunkSize, TArray<uint32>& OutChunkOffsets,
		const uint32 InFirstChunkMaxLengthAudioFrames, const uint32 InMaxChunkLengthAudioFrames) const
	{
		// Split using the seek-table if we have one.
		if (const bool bRequiresSeekTable = InFormat->RequiresStreamingSeekTable())
		{
			// Because the extract call modifies the source buffer inline (to remove the embedded seek table) we must keep a copy as if
			// the call was to fail to split the calling code must contend with the original.
			FAudioCookOutputs InSrcBufferCopy = InCookOutputs;
		
			IAudioFormat::FSeekTable Table;			
			if (!ensureMsgf(InFormat->ExtractSeekTableForStreaming(InSrcBufferCopy.EncodedData, Table), TEXT("SoundWave: '%s' requires a Seektable, but doesn't contain one."), *SoundWaveFullName))
			{
				return false;
			}

			// A limitation of our pipeline requires that we decode a minimum of MONO_PCM_BUFFER_SAMPLES (8k) for our first chunk.
			// So don't allow our first chunk size in frames be less than that.
			int32 FirstChunkMaxLengthAudioFrames = InFirstChunkMaxLengthAudioFrames;
			if (FirstChunkMaxLengthAudioFrames > 0)
			{
				FirstChunkMaxLengthAudioFrames = FMath::Max(MONO_PCM_BUFFER_SAMPLES, FirstChunkMaxLengthAudioFrames);
			}

			return SplitUsingSeekTable(Table, InSrcBufferCopy,OutBuffers,InFirstChunkMaxSize,InMaxChunkSize, OutChunkOffsets, FirstChunkMaxLengthAudioFrames, InMaxChunkLengthAudioFrames);
		}

		// otherwise... ask the format to split.
		return InFormat->SplitDataForStreaming(InCookOutputs.EncodedData, OutBuffers, InFirstChunkMaxSize, InMaxChunkSize);
	}

	/** Build the streamed audio. This function is safe to call from any thread. */
	void BuildStreamedAudio()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildStreamedAudio);

		if (bIsProcedural)
		{
			return;
		}

		DerivedData->Chunks.Empty();

		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		const IAudioFormat* AudioFormat = NULL;
		if (TPM)
		{
			AudioFormat = TPM->FindAudioFormat(AudioFormatName);
		}

		if (AudioFormat)
		{
			DerivedData->AudioFormat = AudioFormatName;

			FAudioCookOutputs CookOutputs;
			if (GetCompressedData(CookOutputs))
			{
				TArray<TArray<uint8>> ChunkBuffers;

				// Set the ideal chunk size to be 256k to optimize for data reads on console.
				int32 MaxChunkSizeForCurrentWave = 256 * 1024;
				
				// By default, the zeroth chunk's max size is the same as the other chunks.
				int32 ZerothChunkSize = MaxChunkSizeForCurrentWave;

				const int32 MinimumChunkSize = AudioFormat->GetMinimumSizeForInitialChunk(AudioFormatName, CookOutputs.EncodedData);
				const bool bForceLegacyStreamChunking = bIsStreaming && CompressionOverrides && CompressionOverrides->StreamCachingSettings.bForceLegacyStreamChunking;

				// If the zeroth size for this sound wave was overridden, use that:
				if (ZerothChunkSizeSoundWaveOverride > 0)
				{
					ZerothChunkSize = FMath::Max(MinimumChunkSize, ZerothChunkSizeSoundWaveOverride);
				}
				else
				{
					// Ensure that the minimum chunk size is nonzero if our compressed buffer is not empty.
					checkf(CookOutputs.EncodedData.Num() == 0 || MinimumChunkSize != 0, TEXT("To use Load On Demand, please override GetMinimumSizeForInitialChunk"));

					//
					if (bForceLegacyStreamChunking)
					{
						int32 LegacyZerothChunkSize = CompressionOverrides->StreamCachingSettings.ZerothChunkSizeForLegacyStreamChunkingKB * 1024;
						if (LegacyZerothChunkSize == 0)
						{
							LegacyZerothChunkSize = MaxChunkSizeForCurrentWave;
						}

						ZerothChunkSize = LegacyZerothChunkSize;
					}
					else
					{
						// Otherwise if we're using Audio Stream Caching, the first chunk should be as small as possible:
						ZerothChunkSize = MinimumChunkSize;
					}
				}

				if (!bForceLegacyStreamChunking)
				{
					// Use the chunk size for this duration:
					MaxChunkSizeForCurrentWave = FPlatformCompressionUtilities::GetMaxChunkSizeForCookOverrides(CompressionOverrides);

					// observe the override chunk size now that we have set the 
					const int32 MaxChunkSizeOverrideBytes = CompressionOverrides->StreamCachingSettings.MaxChunkSizeOverrideKB * 1024;
					if (MaxChunkSizeOverrideBytes > 0)
					{
						MaxChunkSizeForCurrentWave = FMath::Min(MaxChunkSizeOverrideBytes, MaxChunkSizeForCurrentWave);
					}

				}
				

				check(ZerothChunkSize != 0 && MaxChunkSizeForCurrentWave != 0);

				// If platform is configured to so, optionally inline the first Audio chunk and size it with the specific requirements.
				// Value of zero will do nothing.
				const bool bInlineFirstAudioChunk = CompressionOverrides->bInlineFirstAudioChunk && !bForceLegacyStreamChunking;
				uint32 MaxLengthOfFirstAudioChunkInFrames = MAX_uint32;
				if (bInlineFirstAudioChunk)
				{
					MaxLengthOfFirstAudioChunkInFrames = CookOutputs.SampleRate * SizeOfFirstAudioChunkInSeconds;
				}

				TArray<uint32> ChunkOffsets;
				if (SplitDataForStreaming(AudioFormat, CookOutputs, ChunkBuffers, ZerothChunkSize,
				                          MaxChunkSizeForCurrentWave, ChunkOffsets, MaxLengthOfFirstAudioChunkInFrames,
				                          MAX_uint32))
				{
					UE_LOGF(LogAudio, Display, "Chunk stats for (%ls: Duration=%2.2f secs, Channels=%d), Settings(FirstChunkSize=%u frames, InlineFirst=%ls, MaxChunkSize=%d), Chunks=%d, Chunk0=%d bytes, Chunk1=%d bytes, ChunkN=%d)",
						*SoundWaveFullName,(float)CookOutputs.NumFrames / CookOutputs.SampleRate, CookOutputs.NumChannels,
						MaxLengthOfFirstAudioChunkInFrames, ToCStr(LexToString(bInlineFirstAudioChunk)), MaxChunkSizeForCurrentWave,
						ChunkBuffers.Num(),
						ChunkBuffers.IsValidIndex(0) ? ChunkBuffers[0].Num() : 0,
						ChunkBuffers.IsValidIndex(1) ? ChunkBuffers[1].Num() : 0,
						ChunkBuffers.IsValidIndex(2) ? ChunkBuffers[2].Num() : 0
						);
					
					if (ChunkBuffers.Num() > 32)
					{
						UE_LOGF(LogAudio, Display, "Sound Wave %ls is very large, requiring %d chunks.", *SoundWaveFullName, ChunkBuffers.Num());
					}

					if (ChunkBuffers.Num() > 0)
					{
						// The zeroth chunk should not be zero-padded.
						int32 AudioDataSize = ChunkBuffers[0].Num();

						//FStreamedAudioChunk* NewChunk = new(DerivedData->Chunks) FStreamedAudioChunk();
						int32 ChunkIndex = DerivedData->Chunks.Add(new FStreamedAudioChunk());
						FStreamedAudioChunk* NewChunk = &(DerivedData->Chunks[ChunkIndex]);

						// Store both the audio data size and the data size so decoders will know what portion of the bulk data is real audio
						NewChunk->AudioDataSize = AudioDataSize;
						NewChunk->DataSize = AudioDataSize;
						NewChunk->SeekOffsetInAudioFrames = ChunkOffsets.IsValidIndex(ChunkIndex) ? ChunkOffsets[ChunkIndex] : INDEX_NONE;

						if (NewChunk->BulkData.IsLocked())
						{
							UE_LOGF(LogAudioDerivedData, Warning, "While building split chunk for streaming: Raw PCM data already being written to. Chunk Index: 0 SoundWave: %ls ", *SoundWaveFullName);
						}

						NewChunk->BulkData.Lock(LOCK_READ_WRITE);
						void* NewChunkData = NewChunk->BulkData.Realloc(NewChunk->AudioDataSize);
						FMemory::Memcpy(NewChunkData, ChunkBuffers[0].GetData(), AudioDataSize);
						NewChunk->BulkData.Unlock();
					}

					// Zero-pad the rest of the chunks here:
					for (int32 ChunkIndex = 1; ChunkIndex < ChunkBuffers.Num(); ++ChunkIndex)
					{
						// Zero pad the reallocation if the chunk isn't precisely the max chunk size to keep the reads aligned to MaxChunkSize
						int32 AudioDataSize = ChunkBuffers[ChunkIndex].Num();
						check(AudioDataSize != 0);
						ensureMsgf(AudioDataSize <= MaxChunkSizeForCurrentWave, TEXT("Chunk is overbudget by %d bytes"), AudioDataSize - MaxChunkSizeForCurrentWave);

						int32 ZeroPadBytes = 0;

						if (bForceLegacyStreamChunking)
						{
							// padding when stream caching is enabled will significantly bloat the amount of space soundwaves take up on disk.
							ZeroPadBytes = FMath::Max(MaxChunkSizeForCurrentWave - AudioDataSize, 0);
						}

						FStreamedAudioChunk* NewChunk = new FStreamedAudioChunk();
						DerivedData->Chunks.Add(NewChunk);

						// Store both the audio data size and the data size so decoders will know what portion of the bulk data is real audio
						NewChunk->AudioDataSize = AudioDataSize;
						NewChunk->DataSize = AudioDataSize + ZeroPadBytes;
						NewChunk->SeekOffsetInAudioFrames = ChunkOffsets.IsValidIndex(ChunkIndex) ? ChunkOffsets[ChunkIndex] : INDEX_NONE;

						// If this is the first chunk of Audio, ask that we inline it when we serialize.
						// NOTE we should only do this if we've been given a size greater than 0 to put there
						// otherwise the chunk will use the normal size boundary.
						const bool bIsFirstAudioChunk = NewChunk->SeekOffsetInAudioFrames == 0;
						NewChunk->bInlineChunk = bIsFirstAudioChunk && bInlineFirstAudioChunk && SizeOfFirstAudioChunkInSeconds > 0;
						
						if (NewChunk->BulkData.IsLocked())
						{
							UE_LOGF(LogAudioDerivedData, Warning, "While building split chunk for streaming: Raw PCM data already being written to. Chunk Index: %d SoundWave: %ls ", ChunkIndex, *SoundWaveFullName);
						}

						NewChunk->BulkData.Lock(LOCK_READ_WRITE);

						void* NewChunkData = NewChunk->BulkData.Realloc(NewChunk->DataSize);
						FMemory::Memcpy(NewChunkData, ChunkBuffers[ChunkIndex].GetData(), AudioDataSize);

						// if we are padding,
						if (ZeroPadBytes > 0)
						{
							// zero out the end of ChunkData (after the audio data ends).
							FMemory::Memzero((uint8*)NewChunkData + AudioDataSize, ZeroPadBytes);
						}

						NewChunk->BulkData.Unlock();
					}
				}
				else
				{
					// Could not split so copy compressed data into a single chunk
					FStreamedAudioChunk* NewChunk = new FStreamedAudioChunk();
					DerivedData->Chunks.Add(NewChunk);
					NewChunk->DataSize = CookOutputs.EncodedData.Num();
					NewChunk->AudioDataSize = NewChunk->DataSize;

					if (NewChunk->BulkData.IsLocked())
					{
						UE_LOGF(LogAudioDerivedData, Warning, "While building single-chunk streaming SoundWave: Raw PCM data already being written to. SoundWave: %ls ", *SoundWaveFullName);
					}

					NewChunk->BulkData.Lock(LOCK_READ_WRITE);
					void* NewChunkData = NewChunk->BulkData.Realloc(CookOutputs.EncodedData.Num());
					FMemory::Memcpy(NewChunkData, CookOutputs.EncodedData.GetData(), CookOutputs.EncodedData.Num());
					NewChunk->BulkData.Unlock();
				}

				// Store it in the cache.
				// @todo: This will remove the streaming bulk data, which we immediately reload below!
				// Should ideally avoid this redundant work, but it only happens when we actually have
				// to build the compressed audio, which should only ever be once.
				this->BytesCached = PutDerivedDataInCache(DerivedData, KeySuffix, SoundWavePath);

				check(this->BytesCached != 0);
			}
			else
			{
				UE_LOGF(LogAudio, Display, "Failed to retrieve compressed data for format %ls and soundwave %ls.",
					   *AudioFormatName.GetPlainNameString(),
					   *SoundWavePath
					);
			}
		}

		if (DerivedData->Chunks.Num())
		{
			bool bInlineChunks = (CacheFlags & EStreamedAudioCacheFlags::InlineChunks) != 0;
			bSucceeded = !bInlineChunks || DerivedData->TryInlineChunkData();
		}
		else
		{
			UE_LOGF(LogAudio, Display, "Failed to build %ls derived data for %ls",
				*AudioFormatName.GetPlainNameString(),
				*SoundWavePath
				);
		}
	}

public:
	/** Initialization constructor. */
	FStreamedAudioCacheDerivedDataWorker(
		FStreamedAudioPlatformData* InDerivedData,
		USoundWave* InSoundWave,
		const FPlatformAudioCookOverrides* InCompressionOverrides,
		FName InAudioFormatName,
		uint32 InCacheFlags,
		const ITargetPlatform* InTargetPlatform=nullptr
		)
		: DerivedData(InDerivedData)
		, SoundWavePath(InSoundWave->GetPathName())
		, SoundWaveFullName(InSoundWave->GetFullName())
		, AudioFormatName(InAudioFormatName)
		, CacheFlags(InCacheFlags)
		, CompressionOverrides(InCompressionOverrides)
		, bIsProcedural(InSoundWave->IsA<USoundWaveProcedural>())
		, bIsStreaming(InSoundWave->bStreaming)
		, ZerothChunkSizeSoundWaveOverride(InSoundWave->InitialChunkSize_DEPRECATED)
		, SizeOfFirstAudioChunkInSeconds(InSoundWave->GetSizeOfFirstAudioChunkInSeconds(InTargetPlatform))
	{
		// Gather all USoundWave object inputs to avoid race-conditions that could result when touching the UObject from another thread.
		GetStreamedAudioDerivedDataKeySuffix(*InSoundWave, AudioFormatName, CompressionOverrides, InTargetPlatform, KeySuffix);
		const FName PlatformSpecificFormat = InSoundWave->GetPlatformSpecificFormat(InAudioFormatName, CompressionOverrides);

 		// Fetch compressed data directly from the DDC to ensure thread-safety. Will be async if the compressor is thread-safe.
		FDerivedAudioDataCompressor* DeriveAudioData = new FDerivedAudioDataCompressor(InSoundWave, InAudioFormatName, PlatformSpecificFormat, CompressionOverrides, InTargetPlatform);
		CompressedAudioHandle = GetDerivedDataCacheRef().GetAsynchronous(DeriveAudioData);
	}

	/** Does the work to cache derived data. Safe to call from any thread. */
	void DoWork()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStreamedAudioCacheDerivedDataWorker::DoWork);

		// This scope will let us access any incomplete properties since we are the producer of those properties and
		// we can't wait on ourself without causing a deadlock.
		FStreamedAudioBuildScope StreamedAudioBuildScope(DerivedData);

		TArray<uint8> RawDerivedData;
		bool bForceRebuild = (CacheFlags & EStreamedAudioCacheFlags::ForceRebuild) != 0;
		bool bInlineChunks = (CacheFlags & EStreamedAudioCacheFlags::InlineChunks) != 0;
		bool bForDDC = (CacheFlags & EStreamedAudioCacheFlags::ForDDCBuild) != 0;
		bool bAllowAsyncBuild = (CacheFlags & EStreamedAudioCacheFlags::AllowAsyncBuild) != 0;

		if (!bForceRebuild && GetDerivedDataCacheRef().GetSynchronous(*DerivedData->DerivedDataKey, RawDerivedData, SoundWavePath))
		{
			BytesCached = RawDerivedData.Num();
			FMemoryReader Ar(RawDerivedData, /*bIsPersistent=*/ true);
			DerivedData->Serialize(Ar, NULL);
			bSucceeded = true;
			// Load any streaming (not inline) chunks that are necessary for our platform.
			if (bForDDC)
			{
				for (int32 Index = 0; Index < DerivedData->Chunks.Num(); ++Index)
				{
					if (!DerivedData->GetChunkFromDDC(Index, NULL))
					{
						bSucceeded = false;
						break;
					}
				}
			}
			else if (bInlineChunks)
			{
				bSucceeded = DerivedData->TryInlineChunkData();
			}
			else
			{
				bSucceeded = DerivedData->AreDerivedChunksAvailable(SoundWavePath);
			}
			bLoadedFromDDC = true;
		}

		// Let us try to build asynchronously if allowed to after a DDC fetch failure instead of relying solely on the
		// synchronous finalize to perform all the work.
		if (!bSucceeded && bAllowAsyncBuild)
		{
			// Let Finalize know that we've already tried to build in case we didn't succeed, don't try a second time for nothing.
			bHasBeenBuilt = true;
			BuildStreamedAudio();
		}
	}

	/** Finalize work. Must be called ONLY by the thread that started this task! */
	bool Finalize()
	{
		// if we couldn't get from the DDC or didn't build synchronously, then we have to build now.
		// This is a super edge case that should rarely happen.
		if (!bSucceeded && !bHasBeenBuilt)
		{
			BuildStreamedAudio();
		}

		// Cleanup the async DDC query if needed
		FAudioCookOutputs DummyOutputs;
		GetCompressedData(DummyOutputs);

		return bLoadedFromDDC;
	}

	/** Expose bytes cached for telemetry. */
	uint32 GetBytesCached() const
	{
		return BytesCached;
	}

	/** Expose how the resource was returned for telemetry. */
	bool WasLoadedFromDDC() const
	{
		return bLoadedFromDDC;
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FStreamedAudioCacheDerivedDataWorker, STATGROUP_ThreadPoolAsyncTasks);
	}
};

struct FStreamedAudioAsyncCacheDerivedDataTask : public FAsyncTask<FStreamedAudioCacheDerivedDataWorker>
{
	FStreamedAudioAsyncCacheDerivedDataTask(
		FStreamedAudioPlatformData* InDerivedData,
		USoundWave* InSoundWave,
		const FPlatformAudioCookOverrides* CompressionSettings,
		FName InAudioFormatName,
		uint32 InCacheFlags,
		const ITargetPlatform* InTargetPlatform=nullptr
		)
		: FAsyncTask<FStreamedAudioCacheDerivedDataWorker>(
			InDerivedData,
			InSoundWave,
			CompressionSettings,
			InAudioFormatName,
			InCacheFlags,
			InTargetPlatform
			)
	{
	}
};

void FStreamedAudioPlatformData::Cache(USoundWave& InSoundWave, const FPlatformAudioCookOverrides* CompressionOverrides, FName AudioFormatName, uint32 InFlags, const ITargetPlatform* InTargetPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamedAudioPlatformData::Cache);

	// Flush any existing async task and ignore results.
	FinishCache();

	uint32 Flags = InFlags;

	static bool bForDDC = FString(FCommandLine::Get()).Contains(TEXT("Run=DerivedDataCache"));
	if (bForDDC)
	{
		Flags |= EStreamedAudioCacheFlags::ForDDCBuild;
	}

	bool bForceRebuild = (Flags & EStreamedAudioCacheFlags::ForceRebuild) != 0;
	bool bAsync = (Flags & EStreamedAudioCacheFlags::Async) != 0;
	GetStreamedAudioDerivedDataKey(InSoundWave, AudioFormatName, CompressionOverrides, InTargetPlatform, DerivedDataKey);

	if (bAsync && !bForceRebuild && FSoundWaveCompilingManager::Get().IsAsyncCompilationAllowed(&InSoundWave))
	{
		FQueuedThreadPool* SoundWaveThreadPool = FSoundWaveCompilingManager::Get().GetThreadPool();
		EQueuedWorkPriority BasePriority = FSoundWaveCompilingManager::Get().GetBasePriority(&InSoundWave);

		{
			FWriteScopeLock AsyncTaskScope(AsyncTaskLock.Get());
			check(AsyncTask == nullptr);
			AsyncTask = new FStreamedAudioAsyncCacheDerivedDataTask(this, &InSoundWave, CompressionOverrides, AudioFormatName, Flags, InTargetPlatform);

			// Use the size of the Uncompressed data x3 as guestmate of how much memory will be used.
			const int64 PayloadSize = InSoundWave.RawData.GetPayloadSize() * 3;
			// If there is no payoad size for some reason, use the default of -1.
			const int64 RequiredMemory = PayloadSize > 0 ? PayloadSize : -1;

			AsyncTask->StartBackgroundTask(SoundWaveThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, RequiredMemory, TEXT("AudioDerivedData") );
		}

		if (IsInAudioThread())
		{
			TWeakObjectPtr<USoundWave> WeakSoundWavePtr(&InSoundWave);
			FAudioThread::RunCommandOnGameThread([WeakSoundWavePtr]()
			{
				if (USoundWave* SoundWave = WeakSoundWavePtr.Get())
				{
					FSoundWaveCompilingManager::Get().AddSoundWaves({SoundWave});
				}
			});
		}
		else
		{
			FSoundWaveCompilingManager::Get().AddSoundWaves({&InSoundWave});
		}
	}
	else
	{
		FStreamedAudioCacheDerivedDataWorker Worker(this, &InSoundWave, CompressionOverrides, AudioFormatName, Flags, InTargetPlatform);
		{
			COOK_STAT(auto Timer = AudioCookStats::UsageStats.TimeSyncWork());
			Worker.DoWork();
			Worker.Finalize();
			COOK_STAT(Timer.AddHitOrMiss(Worker.WasLoadedFromDDC() ? FCookStats::CallStats::EHitOrMiss::Hit : FCookStats::CallStats::EHitOrMiss::Miss, Worker.GetBytesCached()));
		}
	}
}

bool FStreamedAudioPlatformData::IsCompiling() const
{
	FReadScopeLock AsyncTaskScope(AsyncTaskLock.Get());
	return AsyncTask != nullptr;
}

bool FStreamedAudioPlatformData::IsAsyncWorkComplete() const
{
	FReadScopeLock AsyncTaskScope(AsyncTaskLock.Get());
	return AsyncTask == nullptr || AsyncTask->IsWorkDone();
}

bool FStreamedAudioPlatformData::IsFinishedCache() const
{
	FReadScopeLock AsyncTaskScope(AsyncTaskLock.Get());
	return AsyncTask == nullptr ? true : false;
}

void FStreamedAudioPlatformData::FinishCache()
{
	FWriteScopeLock AsyncTaskScope(AsyncTaskLock.Get());
	if (AsyncTask)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStreamedAudioPlatformData::FinishCache);
		{
			COOK_STAT(auto Timer = AudioCookStats::UsageStats.TimeAsyncWait());
			AsyncTask->EnsureCompletion();
			FStreamedAudioCacheDerivedDataWorker& Worker = AsyncTask->GetTask();
			Worker.Finalize();
			COOK_STAT(Timer.AddHitOrMiss(Worker.WasLoadedFromDDC() ? FCookStats::CallStats::EHitOrMiss::Hit : FCookStats::CallStats::EHitOrMiss::Miss, Worker.GetBytesCached()));
		}
		delete AsyncTask;
		AsyncTask = nullptr;
	}
}

bool FStreamedAudioPlatformData::RescheduleAsyncTask(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority)
{
	FReadScopeLock AsyncTaskScope(AsyncTaskLock.Get());
	return AsyncTask ? AsyncTask->Reschedule(InThreadPool, InPriority) : false;
}

bool FStreamedAudioPlatformData::WaitAsyncTaskWithTimeout(float InTimeoutInSeconds)
{
	FReadScopeLock AsyncTaskScope(AsyncTaskLock.Get());
	return AsyncTask ? AsyncTask->WaitCompletionWithTimeout(InTimeoutInSeconds) : true;
}

/**
 * Executes async DDC gets for chunks stored in the derived data cache.
 * @param Chunks - Chunks to retrieve.
 * @param FirstChunkToLoad - Index of the first chunk to retrieve.
 * @param OutHandles - Handles to the asynchronous DDC gets.
 */
static void BeginLoadDerivedChunks(TIndirectArray<FStreamedAudioChunk>& Chunks, int32 FirstChunkToLoad, TArray<uint32>& OutHandles)
{
	FDerivedDataCacheInterface& DDC = GetDerivedDataCacheRef();
	OutHandles.AddZeroed(Chunks.Num());
	for (int32 ChunkIndex = FirstChunkToLoad; ChunkIndex < Chunks.Num(); ++ChunkIndex)
	{
		const FStreamedAudioChunk& Chunk = Chunks[ChunkIndex];
		if (!Chunk.DerivedDataKey.IsEmpty())
		{
			OutHandles[ChunkIndex] = DDC.GetAsynchronous(*Chunk.DerivedDataKey, TEXTVIEW("Unknown SoundWave"));
		}
	}
}

bool FStreamedAudioPlatformData::TryInlineChunkData()
{
	TArray<uint32> AsyncHandles;
	TArray<uint8> TempData;
	FDerivedDataCacheInterface& DDC = GetDerivedDataCacheRef();

	BeginLoadDerivedChunks(Chunks, 0, AsyncHandles);
	for (int32 ChunkIndex = 0; ChunkIndex < Chunks.Num(); ++ChunkIndex)
	{
		FStreamedAudioChunk& Chunk = Chunks[ChunkIndex];
		if (Chunk.DerivedDataKey.IsEmpty() == false)
		{
			uint32 AsyncHandle = AsyncHandles[ChunkIndex];
			bool bLoadedFromDDC = false;
			COOK_STAT(auto Timer = AudioCookStats::StreamingChunkUsageStats.TimeAsyncWait());
			DDC.WaitAsynchronousCompletion(AsyncHandle);
			bLoadedFromDDC = DDC.GetAsynchronousResults(AsyncHandle, TempData);
			COOK_STAT(Timer.AddHitOrMiss(bLoadedFromDDC ? FCookStats::CallStats::EHitOrMiss::Hit : FCookStats::CallStats::EHitOrMiss::Miss, TempData.Num()));
			if (bLoadedFromDDC)
			{
				int32 ChunkSize = 0;
				int32 AudioDataSize = 0;
				FMemoryReader Ar(TempData, /*bIsPersistent=*/ true);
				Ar << ChunkSize;
				Ar << AudioDataSize; // Unused for the purposes of this function.

				if (Chunk.BulkData.IsLocked())
				{
					UE_LOGF(LogAudioDerivedData, Warning, "In TryInlineChunkData: Raw PCM data already being written to. Chunk: %d DDC Key: %ls ", ChunkIndex, *DerivedDataKey);
				}

				Chunk.BulkData.Lock(LOCK_READ_WRITE);
				void* ChunkData = Chunk.BulkData.Realloc(ChunkSize);
				Ar.Serialize(ChunkData, ChunkSize);
				Chunk.BulkData.Unlock();
				Chunk.DerivedDataKey.Empty();
			}
			else
			{
				return false;
			}
			TempData.Reset();
		}
	}
	return true;
}

#endif //WITH_EDITORONLY_DATA

FStreamedAudioPlatformData::FStreamedAudioPlatformData()
#if WITH_EDITORONLY_DATA
	: AsyncTask(nullptr)
#endif // #if WITH_EDITORONLY_DATA
{
}

FStreamedAudioPlatformData::~FStreamedAudioPlatformData()
{
#if WITH_EDITORONLY_DATA
	FWriteScopeLock AsyncTaskScope(AsyncTaskLock.Get());
	if (AsyncTask)
	{
		AsyncTask->EnsureCompletion();
		delete AsyncTask;
		AsyncTask = nullptr;
	}
#endif
}

int32 FStreamedAudioPlatformData::DeserializeChunkFromDDC(TArray<uint8> TempData, FStreamedAudioChunk &Chunk, int32 ChunkIndex, uint8** &OutChunkData)
{
	int32 ChunkSize = 0;
	FMemoryReader Ar(TempData, /*bIsPersistent=*/ true);
	int32 AudioDataSize = 0;
	Ar << ChunkSize;
	Ar << AudioDataSize;

#if WITH_EDITORONLY_DATA
	ensureAlwaysMsgf((ChunkSize == Chunk.DataSize && AudioDataSize == Chunk.AudioDataSize),
		TEXT("Chunk %d of %s SoundWave has invalid data in the DDC. Got %d bytes, expected %d. Audio Data was %d bytes but we expected %d bytes. Key=%s"),
		ChunkIndex,
		*AudioFormat.ToString(),
		ChunkSize,
		Chunk.DataSize,
		AudioDataSize,
		Chunk.AudioDataSize,
		*Chunk.DerivedDataKey
	);
#endif

	if (OutChunkData)
	{
		if (*OutChunkData == NULL)
		{
			*OutChunkData = static_cast<uint8*>(FMemory::Malloc(ChunkSize));
		}
		Ar.Serialize(*OutChunkData, ChunkSize);
	}

	return AudioDataSize;
}

int32 FStreamedAudioPlatformData::GetChunkFromDDC(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded /* = false */)
{
	if (GetNumChunks() == 0)
	{
		UE_LOGF(LogAudioDerivedData, Display, "No streamed audio chunks found!");
		return 0;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamedAudioPlatformData::GetChunkFromDDC);

	// if bMakeSureChunkIsLoaded is true, we don't actually know the size of the chunk's bulk data,
	// so it will need to be allocated in GetCopy.
	check(!bMakeSureChunkIsLoaded || (OutChunkData && (*OutChunkData == nullptr)));

	bool bCachedChunk = false;
	check(ChunkIndex < GetNumChunks());
	FStreamedAudioChunk& Chunk = GetChunks()[ChunkIndex];

	int32 ChunkDataSize = 0;

#if WITH_EDITORONLY_DATA
	TArray<uint8> TempData;

	// Begin async DDC retrieval
	FDerivedDataCacheInterface& DDC = GetDerivedDataCacheRef();
	uint32 AsyncHandle = 0;
	if (!Chunk.DerivedDataKey.IsEmpty())
	{
		if (bMakeSureChunkIsLoaded)
		{
			if (DDC.GetSynchronous(*Chunk.DerivedDataKey, TempData, TEXTVIEW("Unknown SoundWave")))
			{
				ChunkDataSize = DeserializeChunkFromDDC(TempData, Chunk, ChunkIndex, OutChunkData);
			}
		}
		else
		{
			AsyncHandle = DDC.GetAsynchronous(*Chunk.DerivedDataKey, TEXTVIEW("Unknown SoundWave"));
		}
	}
	else if (Chunk.bLoadedFromCookedPackage)
	{
		if (Chunk.BulkData.IsBulkDataLoaded() || bMakeSureChunkIsLoaded)
		{
			if (OutChunkData)
			{
				ChunkDataSize = Chunk.BulkData.GetBulkDataSize();
				Chunk.GetCopy((void**)OutChunkData);
			}
		}
	}

	// Wait for async DDC to complete
	// Necessary otherwise we will return a ChunkDataSize of 0 which is considered a failure by most callers and will trigger rebuild. 
	if (Chunk.DerivedDataKey.IsEmpty() == false && AsyncHandle)
	{
		DDC.WaitAsynchronousCompletion(AsyncHandle);
		if (DDC.GetAsynchronousResults(AsyncHandle, TempData))
		{
			ChunkDataSize = DeserializeChunkFromDDC(TempData, Chunk, ChunkIndex, OutChunkData);
		}
	}
#else // #if WITH_EDITORONLY_DATA
	// Load chunk from bulk data if available. If the chunk is not loaded, GetCopy will load it synchronously.
	if (Chunk.BulkData.IsBulkDataLoaded() || bMakeSureChunkIsLoaded)
	{
		if (OutChunkData)
		{
			ChunkDataSize = Chunk.BulkData.GetBulkDataSize();
			Chunk.GetCopy((void**)OutChunkData);
		}
	}
#endif // #if WITH_EDITORONLY_DATA
	return ChunkDataSize;
}

#if WITH_EDITORONLY_DATA
bool FStreamedAudioPlatformData::AreDerivedChunksAvailable(FStringView InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamedAudioPlatformData::AreDerivedChunksAvailable);

	TArray<FString> ChunkKeys;
	for (const FStreamedAudioChunk& Chunk : Chunks)
	{
		if (!Chunk.DerivedDataKey.IsEmpty())
		{
			ChunkKeys.Add(Chunk.DerivedDataKey);
		}
	}
	
	const bool bAllCachedDataProbablyExists = GetDerivedDataCacheRef().AllCachedDataProbablyExists(ChunkKeys);
	
	if (bAllCachedDataProbablyExists)
	{
		// If this is called from the game thread, try to prefetch chunks locally on background thread
		// to avoid doing high latency remote calls every time we reload this data.
		if (IsInGameThread())
		{
			if (GIOThreadPool)
			{
				FString Context{ InContext };
				AsyncPool(
					*GIOThreadPool,
					[ChunkKeys, Context]()
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(PrefetchAudioChunks);
						GetDerivedDataCacheRef().TryToPrefetch(ChunkKeys, Context);
					},
					nullptr,
					EQueuedWorkPriority::Low
				);
			}
		}
		else
		{
			// Not on the game-thread, prefetch synchronously
			TRACE_CPUPROFILER_EVENT_SCOPE(PrefetchAudioChunks);
			GetDerivedDataCacheRef().TryToPrefetch(ChunkKeys, InContext);
		}
	}

	return bAllCachedDataProbablyExists;
}

bool FStreamedAudioPlatformData::AreDerivedChunksAvailable() const
{
	return AreDerivedChunksAvailable(TEXTVIEW("DerivedAudioChunks"));
}

#endif // #if WITH_EDITORONLY_DATA

void FStreamedAudioPlatformData::Serialize(FArchive& Ar, USoundWave* Owner)
{
#if WITH_EDITORONLY_DATA
	if (Owner)
	{
		Owner->RawDataCriticalSection.Lock();
	}
#endif

	int32 NumChunks = 0;
	if (Ar.IsSaving())
	{
		NumChunks = Chunks.Num();
	}

	Ar << NumChunks;
	Ar << AudioFormat;

#if WITH_EDITOR
	if (Ar.IsCooking() && Ar.IsSaving() && Ar.GetCookContext() && Ar.GetCookContext()->GetCookTagList() && Owner)
	{
		FCookTagList* CookTags = Ar.GetCookContext()->GetCookTagList();
		CookTags->Add(Owner, "StreamingFormat", LexToString(AudioFormat));

		const ITargetPlatform* CookingTarget = Ar.CookingTarget();
		check(CookingTarget);

		const FName Format = CookingTarget->GetWaveFormat(Owner);

		/* note: must have "Diff_" prefix for the diff tool to pick up the tag as important for diffing */
		CookTags->Add(Owner, "Diff_10_RawDataHash", GetSoundWaveRawDataHash(*Owner));
		CookTags->Add(Owner, "Diff_20_WaveTransformHash", FString(GetWaveformTransformationHash(*Owner)));
		CookTags->Add(Owner, "Diff_30_FormatHash", Format.ToString());
		CookTags->Add(Owner, "Diff_40_SoundWaveHash", FString(GetSoundWaveHash(*Owner, CookingTarget)));
		CookTags->Add(Owner, "Diff_50_CompressedDataGuid", Owner->CompressedDataGuid.ToString());
	}
#endif

	if (Ar.IsLoading())
	{
		check(!AudioFormat.IsNone());
		check(NumChunks >= 0);

		Chunks.Empty(NumChunks);
		for (int32 ChunkIndex = 0; ChunkIndex < NumChunks; ++ChunkIndex)
		{
			Chunks.Add(new FStreamedAudioChunk());
		}
	}

	for (int32 ChunkIndex = 0; ChunkIndex < Chunks.Num(); ++ChunkIndex)
	{
		Chunks[ChunkIndex].Serialize(Ar, Owner, ChunkIndex);
	}
	

#if WITH_EDITORONLY_DATA
	if (Owner)
	{
		Owner->RawDataCriticalSection.Unlock();
	}
#endif
}

/**
 * Helper class to display a status update message in the editor.
 */
class FAudioStatusMessageContext : FScopedSlowTask
{
public:

	/**
	 * Updates the status message displayed to the user.
	 */
	explicit FAudioStatusMessageContext( const FText& InMessage )
	 : FScopedSlowTask(1, InMessage, GIsEditor && !IsRunningCommandlet())
	{
		UE_LOGF(LogAudioDerivedData, Display, "%ls", *InMessage.ToString());
	}
};

/**
* Function used for resampling a USoundWave's WaveData, which is assumed to be int16 here:
*/
static void ResampleWaveData(Audio::FAlignedFloatBuffer& WaveData, int32 NumChannels, float SourceSampleRate, float DestinationSampleRate)
{
	double StartTime = FPlatformTime::Seconds();

	// Set up temporary output buffers:
	Audio::FAlignedFloatBuffer ResamplerOutputData;

	int32 NumSamples = WaveData.Num();
	
	// set up converter input params:
	Audio::FResamplingParameters ResamplerParams = {
		Audio::EResamplingMethod::BestSinc,
		NumChannels,
		SourceSampleRate,
		DestinationSampleRate,
		WaveData
	};

	// Allocate enough space in output buffer for the resulting audio:
	ResamplerOutputData.AddUninitialized(Audio::GetOutputBufferSize(ResamplerParams));
	Audio::FResamplerResults ResamplerResults;
	ResamplerResults.OutBuffer = &ResamplerOutputData;

	// Resample:
	if (Audio::Resample(ResamplerParams, ResamplerResults))
	{
		// resize WaveData buffer and convert back to int16:
		int32 NumSamplesGenerated = ResamplerResults.OutputFramesGenerated * NumChannels;

		WaveData.SetNum(NumSamplesGenerated);
		FMemory::Memcpy(WaveData.GetData(), ResamplerOutputData.GetData(), NumSamplesGenerated * sizeof(float));
	}
	else
	{
		UE_LOGF(LogAudioDerivedData, Error, "Resampling operation failed.");
	}

	double TimeDelta = FPlatformTime::Seconds() - StartTime;
	UE_LOGF(LogAudioDerivedData, Display, "Resampling file from %d to %d took %2.2f seconds.", (int32)SourceSampleRate, (int32)DestinationSampleRate, TimeDelta);
}

struct FAudioCookInputs
{
	FString                     SoundName;
	FName                       BaseFormat;
	FName                       HashedFormat;
	const IAudioFormat*         Compressor;
	FGuid                       CompressedDataGuid;
#if WITH_EDITORONLY_DATA
	FString                     SoundFullName;
	TArray<int32>               ChannelOffsets;
	TArray<int32>               ChannelSizes;
	bool                        bIsASourceBus;
	bool                        bIsSoundWaveProcedural;
	int32                       CompressionQuality;
	float                       SampleRateOverride = -1.0f;
	bool                        bIsStreaming;
	float                       CompressionQualityModifier;
	FString						SoundWaveHash;
		
	TArray<Audio::FTransformationPtr> WaveTransformations;

	// Those are the only refs we keep on the actual USoundWave until
	// we have a mechanism in place to reference a bulkdata using a 
	// copy-on-write mechanism that doesn't require us to make
	// a copy in memory to get immutability.
	FCriticalSection& BulkDataCriticalSection;
	USoundWave::FEditorAudioBulkData& BulkData;
#endif

	FAudioCookInputs(USoundWave* InSoundWave, FName InBaseFormat, FName InHashFormat, const FPlatformAudioCookOverrides* InCookOverrides, const ITargetPlatform* InTargetPlatform)
		: SoundName(InSoundWave->GetName())
		, BaseFormat(InBaseFormat)
		, HashedFormat(InHashFormat)
		, CompressedDataGuid(InSoundWave->CompressedDataGuid)
#if WITH_EDITORONLY_DATA
		, SoundFullName(InSoundWave->GetFullName())
		, ChannelOffsets(InSoundWave->ChannelOffsets)
		, ChannelSizes(InSoundWave->ChannelSizes)
		, bIsASourceBus(InSoundWave->IsA<USoundSourceBus>())
		, bIsSoundWaveProcedural(InSoundWave->IsA<USoundWaveProcedural>())
		, CompressionQuality(InSoundWave->GetCompressionQuality())
		, CompressionQualityModifier(InCookOverrides ? InCookOverrides->CompressionQualityModifier : 1.0f)
		, SoundWaveHash(GetSoundWaveHash(*InSoundWave, InTargetPlatform))
		, WaveTransformations(InSoundWave->CreateTransformations())
		, BulkDataCriticalSection(InSoundWave->RawDataCriticalSection)
		, BulkData(InSoundWave->RawData)
#endif
	{
#if WITH_EDITORONLY_DATA
		checkf(IsInGameThread() || IsInAudioThread() || IsInAsyncLoadingThread(), TEXT("FAudioCookInputs creation must happen on the game-thread or audio-thread as it reads from many non-thread safe properties of USoundWave"));

#if FORCE_RESAMPLE
		FPlatformAudioCookOverrides NewCompressionOverrides = FPlatformAudioCookOverrides();
		NewCompressionOverrides.bResampleForDevice = true;
		if (InCookOverrides == nullptr)
		{
			InCookOverrides = &NewCompressionOverrides;
		}
#endif //FORCE_RESAMPLE

		if (InCookOverrides && InCookOverrides->bResampleForDevice)
		{
			SampleRateOverride = InSoundWave->GetSampleRateForCompressionOverrides(InCookOverrides);
		}

		// without overrides, we don't know the target platform's name to be able to look up, and passing nullptr will use editor platform's settings, which could be wrong
		// @todo: Pass in TargetPlatform/PlatformName maybe?
		bIsStreaming = InSoundWave->IsStreaming(InCookOverrides ? *InCookOverrides : FPlatformAudioCookOverrides());
#endif // WITH_EDITORONLY_DATA

		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		if (TPM)
		{
			Compressor = TPM->FindAudioFormat(InBaseFormat);
		}
	}
};
#if WITH_EDITORONLY_DATA

static int32 CalculateModifiedCompressionQuality(int32 InQuality, float InQualityModifier)
{
	float ModifiedCompressionQuality = (float)InQuality * InQualityModifier;
	return FMath::Clamp<int32>(FMath::FloorToInt(ModifiedCompressionQuality), 1, 100);
}
static TOptional<int32> GetResampleRate(const IAudioFormat* InCompressor, const int32 InSampleRateOverride,
	const FName InFormatName, const int32 InCurrentSampleRate)
{
	using namespace AudioDerivedDataPrivate;
	if (!ensure(InCompressor))
	{
		return {};
	}

	// Start with the rate of the wave.
	int32 ResampleRate = InCurrentSampleRate;
	
	// Inputs specify an override?
	if (InSampleRateOverride > 0 && InSampleRateOverride != InCurrentSampleRate)
	{
		ResampleRate = InSampleRateOverride;

		static bool bLogOnce = false;
		if (!InCompressor->IsSampleRateSupported(ResampleRate) && !bLogOnce)
		{
			UE_LOGF(LogAudio, Warning,  "SampleRate Override '%d' not supported by AudioFormat '%ls'", InSampleRateOverride, *InFormatName.ToString());
			bLogOnce = true;
		}
	}

	// Do we support this current rate/override?
	if (!InCompressor->IsSampleRateSupported(ResampleRate))
	{
		// Ask for the rates supported by this format.
		TArray<int32> SupportedRates(InCompressor->GetSupportedSampleRates());

		// If we've failed the "IsSampleRateSupported" check above, we should have some rate we support defined here.
		checkf(SupportedRates.Num() > 0, TEXT("AudioFormat=%s, returned no supported rates, but rejected our current rate=%d"),
			*InFormatName.ToString(), InCurrentSampleRate);
		
		// Just in case, sort the item, as we binary search them.
		SupportedRates.Sort();

		const int32 MaxSupportedRate = SupportedRates.Last();
		const int32 MinSupportedRate = SupportedRates[0];

		// Special case for just one entry.
		if (SupportedRates.Num() == 1)
		{
			ResampleRate = SupportedRates.Last();
		}		
		// < Min
		else if (ResampleRate < MinSupportedRate)
		{
			ResampleRate = MinSupportedRate;
		}
		// > Max.
		else if (ResampleRate > MaxSupportedRate)
		{
			ResampleRate = MaxSupportedRate;	
		}
		// ... Find first greatest.
		else
		{
			const int32 Index = Algo::LowerBound(SupportedRates, ResampleRate);
			check(SupportedRates.IsValidIndex(Index));
			ResampleRate = SupportedRates[Index];
		}

		// We should have found something now.
		ensure(InCompressor->IsSampleRateSupported(ResampleRate));
	}

	// If we're different return the new rate.
	if (ResampleRate != InCurrentSampleRate && ResampleRate > 0)
	{
		return ResampleRate;
	}

	// Don't resample if the rate is the same at the current.
	return {};
}

/**
 * Cook a simple mono or stereo wave
 */
static void CookSimpleWave(const FAudioCookInputs& Inputs, FAudioCookOutputs& Outputs)
{
	// Warning: Existing released assets should maintain bitwise exact encoded audio
	// in order to minimize patch sizes. Changing anything in this function can 
	// change the final encoded values and result in large unintended patches. 
	
	TRACE_CPUPROFILER_EVENT_SCOPE(CookSimpleWave);

	FWaveModInfo WaveInfo;
	TArray<uint8> Input;
	check(!Outputs.EncodedData.Num());

	bool bWasLocked = false;

	FScopeLock ScopeLock(&Inputs.BulkDataCriticalSection);

	TFuture<FSharedBuffer> FutureBuffer = Inputs.BulkData.GetPayload();
	
	const uint8* RawWaveData = (const uint8*)FutureBuffer.Get().GetData(); // Will block 
	int32 RawDataSize = FutureBuffer.Get().GetSize();

	if (!Inputs.BulkData.HasPayloadData())
	{
		UE_LOGF(LogAudioDerivedData, Warning, "LPCM data failed to load for sound %ls", *Inputs.SoundFullName);
	}
	else if (!WaveInfo.ReadWaveHeader(RawWaveData, RawDataSize, 0))
	{
		// If we failed to parse the wave header, it's either because of an invalid bitdepth or channel configuration.
		UE_LOGF(LogAudioDerivedData, Warning, "Only mono or stereo 16 bit waves allowed: %ls (%d bytes)", *Inputs.SoundFullName, RawDataSize);
	}
	else
	{
		Input.AddUninitialized(WaveInfo.SampleDataSize);
		FMemory::Memcpy(Input.GetData(), WaveInfo.SampleDataStart, WaveInfo.SampleDataSize);
	}

	if(!Input.Num())
	{
		UE_LOGF(LogAudioDerivedData, Warning, "Can't cook %ls because there is no source LPCM data", *Inputs.SoundFullName );
		return;
	}
	
	int32 WaveSampleRate = *WaveInfo.pSamplesPerSec;
	int32 NumChannels = *WaveInfo.pChannels;
	int32 NumBytes = Input.Num();
	int32 NumSamples = NumBytes / sizeof(int16);

	const TOptional<int32> ResampleRate = GetResampleRate(Inputs.Compressor, Inputs.SampleRateOverride, Inputs.BaseFormat, WaveSampleRate);
	const bool bNeedsResample = ResampleRate.IsSet(); 
	const bool bNeedsToApplyWaveTransformation = (Inputs.WaveTransformations.Num() > 0);

	// Only convert PCM wave data to float if needed. The conversion alters the sample
	// values enough to produce different results in the final encoded data. 
	const bool bNeedsFloatConversion = bNeedsResample || bNeedsToApplyWaveTransformation;

	if (bNeedsFloatConversion)
	{
		// To float for processing
		Audio::FAlignedFloatBuffer InputFloatBuffer;
		InputFloatBuffer.SetNumUninitialized(NumSamples);
		
		AudioDerivedDataPrivate::ArrayPcm16ToFloat(MakeArrayView((int16*)Input.GetData(), NumSamples), InputFloatBuffer);

		// Run any transformations
		if(bNeedsToApplyWaveTransformation)
		{
			Audio::FWaveformTransformationWaveInfo TransformationInfo;

			TransformationInfo.Audio = &InputFloatBuffer;
			TransformationInfo.NumChannels = NumChannels;
			TransformationInfo.SampleRate = WaveSampleRate;
			
			for(const Audio::FTransformationPtr& Transformation : Inputs.WaveTransformations)
			{
				Transformation->ProcessAudio(TransformationInfo);
			}

			UE_CLOGF(WaveSampleRate != TransformationInfo.SampleRate, LogAudioDerivedData, Warning, "Wave transformations which alter the sample rate are not supported. Cooked audio for %ls may be incorrect", *Inputs.SoundFullName);
			WaveSampleRate = TransformationInfo.SampleRate;

			UE_CLOGF(NumChannels != TransformationInfo.NumChannels, LogAudioDerivedData, Error, "Wave transformations which alter number of channels are not supported. Cooked audio for %ls may be incorrect", *Inputs.SoundFullName);
			NumChannels = TransformationInfo.NumChannels;

			NumSamples = InputFloatBuffer.Num();
		}
		
		// Resample if necessary
		if (bNeedsResample)
		{
			ResampleWaveData(InputFloatBuffer, NumChannels, WaveSampleRate, *ResampleRate);
			
			WaveSampleRate = *ResampleRate;
			NumSamples = InputFloatBuffer.Num();
		}

		// Clip Normalize
		const float MaxValue = Audio::ArrayMaxAbsValue(InputFloatBuffer);
		if (MaxValue > 1.0f)
		{
			UE_LOGF(LogAudioDerivedData, Display, "Audio clipped during cook: This asset will be normalized by a factor of 1/%f. Consider attenuating the above asset.", MaxValue);

			Audio::ArrayMultiplyByConstantInPlace(InputFloatBuffer, 1.f / MaxValue);
		}

		// back to PCM
		NumBytes = NumSamples * sizeof(int16);
		Input.SetNumUninitialized(NumBytes);
		
		AudioDerivedDataPrivate::ArrayFloatToPcm16(InputFloatBuffer, MakeArrayView((int16*)Input.GetData(), NumSamples));
	}

	// Compression Quality
	FSoundQualityInfo QualityInfo = { 0 };	
	QualityInfo.Quality = CalculateModifiedCompressionQuality(Inputs.CompressionQuality, Inputs.CompressionQualityModifier);

	UE_CLOGF(Inputs.CompressionQuality != QualityInfo.Quality, LogAudioDerivedData,
		Display, "Compression Quality for %ls will be modified from %d to %d, with modifier [%.2f] ",
		*Inputs.SoundFullName, Inputs.CompressionQuality, QualityInfo.Quality, Inputs.CompressionQualityModifier);

	QualityInfo.NumChannels = NumChannels;
	QualityInfo.SampleRate = WaveSampleRate;
	QualityInfo.SampleDataSize = NumBytes;

	QualityInfo.bStreaming = Inputs.bIsStreaming;
	QualityInfo.DebugName = Inputs.SoundFullName;

	static const FName NAME_BINKA("BINKA");
	if (WaveSampleRate > 48000 &&
		Inputs.BaseFormat == NAME_BINKA)
	{
		// We have to do this here because we don't know the name of the wave inside the codec.
		UE_LOGF(LogAudioDerivedData, Warning, "[%ls] High sample rate wave (%d) with Bink Audio - perf waste - high frequencies are discarded by Bink Audio (like most perceptual codecs).", *Inputs.SoundFullName, WaveSampleRate);
	}

	// Cook the data.
	if (!Inputs.Compressor->Cook(Inputs.BaseFormat, Input, QualityInfo, Outputs.EncodedData))
	{
		UE_LOGF(LogAudioDerivedData, Warning, "Cooking sound failed: %ls", *Inputs.SoundFullName);
	}

	// Record the results.
	Outputs.NumChannels = NumChannels;
	Outputs.SampleRate = WaveSampleRate;
	Outputs.NumFrames = NumSamples / NumChannels;
}

/**
 * Cook a multistream (normally 5.1) wave
 */
static void CookSurroundWave(const FAudioCookInputs& Inputs, FAudioCookOutputs& Outputs)
{
	// Warning: Existing released assets should maintain bitwise exact encoded audio
	// in order to minimize patch sizes. Changing anything in this function can 
	// change the final encoded values and result in large unintended patches. 

	TRACE_CPUPROFILER_EVENT_SCOPE(CookSurroundWave);

	check(!Outputs.EncodedData.Num());

	int32					i;
	size_t					SampleDataSize = 0;
	FWaveModInfo			WaveInfo;
	TArray<TArray<uint8> >	SourceBuffers;
	TArray<int32>			RequiredChannels;

	FScopeLock ScopeLock(&Inputs.BulkDataCriticalSection);
	// Lock raw wave data.
	TFuture<FSharedBuffer> FutureBuffer = Inputs.BulkData.GetPayload();
	const uint8* RawWaveData = (const uint8*)FutureBuffer.Get().GetData(); // Will block 
	int32 RawDataSize = FutureBuffer.Get().GetSize();

	if (!RawWaveData || RawDataSize <=0 )
	{
		UE_LOGF(LogAudioDerivedData, Warning, "Cooking surround sound failed: %ls, Failed to load virtualized bulkdata payload", *Inputs.SoundFullName);
		return;
	}

	// Front left channel is the master
	static_assert(SPEAKER_FrontLeft == 0, "Front-left speaker must be first.");

	// loop through channels to find which have data and which are required
	for (i = 0; i < SPEAKER_Count; i++)
	{
		FWaveModInfo WaveInfoInner;

		// Only mono files allowed
		if (WaveInfoInner.ReadWaveHeader(RawWaveData, Inputs.ChannelSizes[i], Inputs.ChannelOffsets[i])
			&& *WaveInfoInner.pChannels == 1)
		{
			if (SampleDataSize == 0)
			{
				// keep wave info/size of first channel data we find
				WaveInfo = WaveInfoInner;
				SampleDataSize = WaveInfo.SampleDataSize;
			}
			switch (i)
			{
				case SPEAKER_FrontLeft:
				case SPEAKER_FrontRight:
				case SPEAKER_LeftSurround:
				case SPEAKER_RightSurround:
					// Must have quadraphonic surround channels
					RequiredChannels.AddUnique(SPEAKER_FrontLeft);
					RequiredChannels.AddUnique(SPEAKER_FrontRight);
					RequiredChannels.AddUnique(SPEAKER_LeftSurround);
					RequiredChannels.AddUnique(SPEAKER_RightSurround);
					break;
				case SPEAKER_FrontCenter:
				case SPEAKER_LowFrequency:
					// Must have 5.1 surround channels
					for (int32 Channel = SPEAKER_FrontLeft; Channel <= SPEAKER_RightSurround; Channel++)
					{
						RequiredChannels.AddUnique(Channel);
					}
					break;
				case SPEAKER_LeftBack:
				case SPEAKER_RightBack:
					// Must have all previous channels
					for (int32 Channel = 0; Channel < i; Channel++)
					{
						RequiredChannels.AddUnique(Channel);
					}
					break;
				default:
					// unsupported channel count
					break;
			}
		}
	}

	if (SampleDataSize == 0)
	{
		UE_LOGF(LogAudioDerivedData, Warning, "Cooking surround sound failed: %ls", *Inputs.SoundFullName );
		return;
	}

	TArray<FWaveModInfo, TInlineAllocator<SPEAKER_Count>> ChannelInfos;
	
	int32 ChannelCount = 0;
	// Extract all the info for channels
	for( i = 0; i < SPEAKER_Count; i++ )
	{
		FWaveModInfo WaveInfoInner;
		if( WaveInfoInner.ReadWaveHeader( RawWaveData, Inputs.ChannelSizes[ i ], Inputs.ChannelOffsets[ i ] )
			&& *WaveInfoInner.pChannels == 1 )
		{
			ChannelCount++;
			ChannelInfos.Add(WaveInfoInner);

			SampleDataSize = FMath::Max<uint32>(WaveInfoInner.SampleDataSize, SampleDataSize);
		}
		else if (RequiredChannels.Contains(i))
		{
			// Add an empty channel for cooking
			ChannelCount++;
			WaveInfoInner.SampleDataSize = 0;

			ChannelInfos.Add(WaveInfoInner);
		}
	}

	// Only allow the formats that can be played back through
	const bool bChannelCountValidForPlayback = ChannelCount == 4 || ChannelCount == 6 || ChannelCount == 7 || ChannelCount == 8;
	
	if( bChannelCountValidForPlayback == false )
	{
		UE_LOGF(LogAudioDerivedData, Warning, "No format available for a %d channel surround sound: %ls", ChannelCount, *Inputs.SoundFullName );
		return;
	}

	// copy channels we need, ensuring all channels are the same size
	for(const FWaveModInfo& ChannelInfo : ChannelInfos)
	{
		TArray<uint8>& Input = *new (SourceBuffers) TArray<uint8>;
		Input.AddZeroed(SampleDataSize);

		if(ChannelInfo.SampleDataSize > 0)
		{
			FMemory::Memcpy(Input.GetData(), ChannelInfo.SampleDataStart, ChannelInfo.SampleDataSize);
		}
	}
	
	int32 WaveSampleRate = *WaveInfo.pSamplesPerSec;
	int32 NumFrames = SampleDataSize / sizeof(int16);

	// bNeedsResample could change if a transformation changes the sample rate

	const TOptional<int32> ResampleRate = GetResampleRate(Inputs.Compressor, Inputs.SampleRateOverride, Inputs.BaseFormat, WaveSampleRate);
	const bool bNeedsResample = ResampleRate.IsSet(); 
	
	const bool bContainsTransformations = Inputs.WaveTransformations.Num() > 0;
	const bool bNeedsDeinterleave = bNeedsResample || bContainsTransformations;

	if(bNeedsDeinterleave)
	{
		// multichannel wav's are stored deinterleaved, but our dsp assumes interleaved
		Audio::FAlignedFloatBuffer InterleavedFloatBuffer;

		Audio::FMultichannelBuffer InputMultichannelBuffer;

		Audio::SetMultichannelBufferSize(ChannelCount, NumFrames, InputMultichannelBuffer);
		
		// convert to float
		for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ChannelIndex++)
		{
			AudioDerivedDataPrivate::ArrayPcm16ToFloat(MakeArrayView((const int16*)(SourceBuffers[ChannelIndex].GetData()), NumFrames), InputMultichannelBuffer[ChannelIndex]);
		}
	
		Audio::ArrayInterleave(InputMultichannelBuffer, InterleavedFloatBuffer);

		// run transformations
		if(bContainsTransformations)
		{
			Audio::FWaveformTransformationWaveInfo TransformationInfo;

			TransformationInfo.Audio = &InterleavedFloatBuffer;
			TransformationInfo.NumChannels = ChannelCount;
			TransformationInfo.SampleRate = WaveSampleRate;
		
			for(const Audio::FTransformationPtr& Transformation : Inputs.WaveTransformations)
			{
				Transformation->ProcessAudio(TransformationInfo);
			}

			UE_CLOGF(WaveSampleRate != TransformationInfo.SampleRate, LogAudioDerivedData, Warning, "Wave transformations which alter the sample rate are not supported. Cooked audio for %ls may be incorrect", *Inputs.SoundFullName);
			UE_CLOGF(ChannelCount != TransformationInfo.NumChannels, LogAudioDerivedData, Error, "Wave transformations which alter number of channels are not supported. Cooked audio for %ls may be incorrect", *Inputs.SoundFullName);

			NumFrames = InterleavedFloatBuffer.Num() / ChannelCount;
		}

		if (bNeedsResample)
		{
			ResampleWaveData(InterleavedFloatBuffer, ChannelCount, WaveSampleRate, *ResampleRate);
			
			WaveSampleRate = *ResampleRate; 
			NumFrames = InterleavedFloatBuffer.Num() / ChannelCount;
		}

		// clip normalize
		const float MaxValue = Audio::ArrayMaxAbsValue(InterleavedFloatBuffer);
		if (MaxValue > 1.0f)
		{
			UE_LOGF(LogAudioDerivedData, Display, "Audio clipped during cook: This asset will be normalized by a factor of 1/%f. Consider attenuating the above asset.", MaxValue);

			Audio::ArrayMultiplyByConstantInPlace(InterleavedFloatBuffer, 1.f / MaxValue);
		}

		Audio::ArrayDeinterleave(InterleavedFloatBuffer, InputMultichannelBuffer, ChannelCount);

		SampleDataSize = NumFrames * sizeof(int16);

		// back to PCM
		for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ChannelIndex++)
		{
			TArray<uint8>& PcmBuffer = SourceBuffers[ChannelIndex];
				
			PcmBuffer.SetNum(SampleDataSize);
			
			AudioDerivedDataPrivate::ArrayFloatToPcm16(InputMultichannelBuffer[ChannelIndex], MakeArrayView((int16*)PcmBuffer.GetData(), NumFrames));
		}
	}

	UE_LOGF(LogAudioDerivedData, Display, "Cooking %d channels for: %ls", ChannelCount, *Inputs.SoundFullName);

	FSoundQualityInfo QualityInfo = { 0 };
	QualityInfo.Quality = CalculateModifiedCompressionQuality(Inputs.CompressionQuality,Inputs.CompressionQualityModifier);

	UE_CLOGF(Inputs.CompressionQuality != QualityInfo.Quality, LogAudioDerivedData,
		Display, "Compression Quality for %ls will be modified from %d to %d, with modifier [%.2f] ",
		*Inputs.SoundFullName, Inputs.CompressionQuality, QualityInfo.Quality, Inputs.CompressionQualityModifier);

	QualityInfo.NumChannels = ChannelCount;
	QualityInfo.SampleRate = WaveSampleRate;
	QualityInfo.SampleDataSize = SampleDataSize;
	QualityInfo.bStreaming = Inputs.bIsStreaming;
	QualityInfo.DebugName = Inputs.SoundFullName;

	static const FName NAME_BINKA("BINKA");
	if (WaveSampleRate > 48000 &&
		Inputs.BaseFormat == NAME_BINKA)
	{
		// We have to do this here because we don't know the name of the wave inside the codec.
		UE_LOGF(LogAudioDerivedData, Warning, "[%ls] High sample rate wave (%d) with Bink Audio - perf waste - high frequencies are discarded by Bink Audio (like most perceptual codecs).", *Inputs.SoundFullName, WaveSampleRate);
	}

	//@todo tighten up the checking for empty results here
	if (!Inputs.Compressor->CookSurround(Inputs.BaseFormat, SourceBuffers, QualityInfo,  Outputs.EncodedData))
	{
		UE_LOGF(LogAudioDerivedData, Warning, "Cooking surround sound failed: %ls", *Inputs.SoundFullName);
	}
 
	// Record the results.
	Outputs.NumChannels = ChannelCount;
	Outputs.SampleRate = WaveSampleRate;
	Outputs.NumFrames = NumFrames;
}

#endif // WITH_EDITORONLY_DATA

int32 FAudioCookOutputs::GetVersion()
{
	static int32 Ver = 2;
	return Ver;
}

bool FAudioCookOutputs::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		Id = GetId(); // Always set to known good ID
		Version = GetVersion();
	}

	Ar << Id;

	// Doses this look sane?
	if (Ar.IsLoading() && Id != GetId())
	{
		return false; 
	}

	// Reject different versions. As we are DDC this indicates an error.
	Ar << Version;
	if (Ar.IsLoading() && Version != GetVersion())
	{
		return false;
	}

	Ar << NumChannels;
	Ar << SampleRate;
	Ar << NumFrames;
	Ar << EncodedData;

	return true;
}

FDerivedAudioDataCompressor::FDerivedAudioDataCompressor(USoundWave* InSoundNode, FName InBaseFormat, FName InHashedFormat, const FPlatformAudioCookOverrides* InCompressionOverrides, const ITargetPlatform* InTargetPlatform)
	: CookInputs(MakeUnique<FAudioCookInputs>(InSoundNode, InBaseFormat, InHashedFormat, InCompressionOverrides, InTargetPlatform))
{
}

const TCHAR* FDerivedAudioDataCompressor::GetVersionString() const
{	
	return AUDIO_DERIVEDDATA_VER;
}

FString FDerivedAudioDataCompressor::GetPluginSpecificCacheKeySuffix() const
{
	int32 FormatVersion = 0xffff; // if the compressor is NULL, this will be used as the version...and in that case we expect everything to fail anyway
	if (CookInputs->Compressor)
	{
		FormatVersion = (int32)CookInputs->Compressor->GetVersion(CookInputs->BaseFormat);
	}

	check(CookInputs->CompressedDataGuid.IsValid());
	FString FormatHash = CookInputs->HashedFormat.ToString().ToUpper();
	FString SoundWaveHash;

#if WITH_EDITORONLY_DATA
	SoundWaveHash = CookInputs->SoundWaveHash;
#endif //WITH_EDITORONLY_DATA
	
	return FString::Printf(TEXT("%s_%04X_%s%s%d"), *FormatHash, FormatVersion, *SoundWaveHash, *CookInputs->CompressedDataGuid.ToString(),
		FAudioCookOutputs::GetVersion());
}

bool FDerivedAudioDataCompressor::IsBuildThreadsafe() const
{
	return AllowAsyncCompression && CookInputs->Compressor ? CookInputs->Compressor->AllowParallelBuild() : false;
}

bool FDerivedAudioDataCompressor::Build(TArray<uint8>& OutData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDerivedAudioDataCompressor::Build);

#if WITH_EDITORONLY_DATA
	if (!CookInputs->Compressor)
	{
		UE_LOGF(LogAudioDerivedData, Warning, "Could not find audio format to cook: %ls", *CookInputs->BaseFormat.ToString());
		return false;
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("AudioFormat"), FText::FromName(CookInputs->BaseFormat));
	Args.Add(TEXT("Hash"), FText::FromName(CookInputs->HashedFormat));
	Args.Add(TEXT("SoundNodeName"), FText::FromString(CookInputs->SoundName));
	FAudioStatusMessageContext StatusMessage(FText::Format(NSLOCTEXT("Engine", "BuildingCompressedAudioTaskStatus", "Building compressed audio format {AudioFormat} hash {Hash} wave {SoundNodeName}..."), Args));

	// these types of sounds do not need cooked data
	if(CookInputs->bIsASourceBus || CookInputs->bIsSoundWaveProcedural)
	{
		return false;
	}

	FAudioCookOutputs CookOutputs;
	if (!CookInputs->ChannelSizes.Num())
	{
		check(!CookInputs->ChannelOffsets.Num());
		CookSimpleWave(*CookInputs, CookOutputs);
	}
	else
	{
		check(CookInputs->ChannelOffsets.Num() == SPEAKER_Count);
		check(CookInputs->ChannelSizes.Num() == SPEAKER_Count);
		CookSurroundWave(*CookInputs, CookOutputs);
	}

	// Serialize to bitstream.
	FMemoryWriter Writer(OutData);
	ensure(CookOutputs.Serialize(Writer));
		
	const uint64 BeforeSize = CookInputs->BulkData.GetPayloadSize();
	const uint64 AfterSize = CookOutputs.EncodedData.Num();
	const float Percent = BeforeSize > 0 ? ((float)AfterSize / BeforeSize) * 100.f : 0.0f;

	// Log message about the completed results
	FFormatNamedArguments Args2;
	Args2.Add(TEXT("AudioFormat"), FText::FromName(CookInputs->BaseFormat));
	Args2.Add(TEXT("SoundNodeName"), FText::FromString(CookInputs->SoundName));	
	Args2.Add(TEXT("BeforeSize"), BeforeSize >> 10);
	Args2.Add(TEXT("AfterSize"),  AfterSize >> 10);
	Args2.Add(TEXT("Percent"), FText::FromString(FString::Printf(TEXT("%2.2f"), Percent)));
	Args2.Add(TEXT("Quality"), FText::FromString(LexToString(CookInputs->CompressionQuality)));
	Args2.Add(TEXT("QualityMod"), FText::FromString(FString::Printf(TEXT("%2.2f"), CookInputs->CompressionQualityModifier)));

	FAudioStatusMessageContext CompressedMessage(FText::Format(NSLOCTEXT("Engine", "BuildingCompressedAudioTaskResults", "{SoundNodeName} compressed to {Percent}% (from {BeforeSize}KB to {AfterSize}KB) with {AudioFormat} at Quality {Quality} with Quality Modifier {QualityMod}"), Args2));

#endif

	return OutData.Num() > 0;
}

FString FDerivedAudioDataCompressor::GetDebugContextString() const
{
#if WITH_EDITORONLY_DATA
	return CookInputs->SoundFullName;
#else
	return CookInputs->SoundName;
#endif
}

/*---------------------------------------
	USoundWave Derived Data functions
---------------------------------------*/

void USoundWave::CleanupCachedRunningPlatformData()
{
	GetWorkingSoundWaveData()->RunningPlatformData = FStreamedAudioPlatformData();
}


void USoundWave::SerializeCookedPlatformData(FArchive& Ar)
{
	if (IsTemplate())
	{
		return;
	}

	DECLARE_SCOPE_CYCLE_COUNTER( TEXT("USoundWave::SerializeCookedPlatformData"), STAT_SoundWave_SerializeCookedPlatformData, STATGROUP_LoadTime );

#if WITH_EDITORONLY_DATA
	if (Ar.IsCooking() && Ar.IsPersistent())
	{
		check(Ar.CookingTarget()->AllowAudioVisualData());

		FName PlatformFormat = Ar.CookingTarget()->GetWaveFormat(this);
		const FPlatformAudioCookOverrides* CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides(*Ar.CookingTarget()->IniPlatformName());
		FString DerivedDataKey;

		GetStreamedAudioDerivedDataKeySuffix(*this, PlatformFormat, CompressionOverrides, Ar.CookingTarget(), DerivedDataKey);

		FStreamedAudioPlatformData *PlatformDataToSave = CookedPlatformData.FindRef(DerivedDataKey);

		if (PlatformDataToSave == NULL)
		{
			PlatformDataToSave = new FStreamedAudioPlatformData();
			PlatformDataToSave->Cache(*this, CompressionOverrides, PlatformFormat, EStreamedAudioCacheFlags::InlineChunks | EStreamedAudioCacheFlags::Async, Ar.CookingTarget());

			CookedPlatformData.Add(DerivedDataKey, PlatformDataToSave);
		}

		PlatformDataToSave->FinishCache();
		PlatformDataToSave->Serialize(Ar, this);
	}
	else
#endif // #if WITH_EDITORONLY_DATA
	{
		check(!FPlatformProperties::IsServerOnly());

		CleanupCachedRunningPlatformData();

		// Don't serialize streaming data on servers, even if this platform supports streaming in theory
		GetWorkingSoundWaveData()->RunningPlatformData.Serialize(Ar, this);
	}
}

#if WITH_EDITORONLY_DATA
void USoundWave::CachePlatformData(bool bAsyncCache)
{
	// don't interact with the DDC if we were loaded from cooked data in editor
	if (bLoadedFromCookedData)
	{
		return;
	}

	FString DerivedDataKey;
	FName AudioFormat = GetWaveFormatForRunningPlatform(*this);
	const FPlatformAudioCookOverrides* CompressionOverrides = GetCookOverridesForRunningPlatform();
	GetStreamedAudioDerivedDataKey(*this, AudioFormat, CompressionOverrides, GetRunningPlatform(),  DerivedDataKey);

	if (GetWorkingSoundWaveData()->RunningPlatformData.DerivedDataKey != DerivedDataKey)
	{
		const uint32 CacheFlags = bAsyncCache ? (EStreamedAudioCacheFlags::Async | EStreamedAudioCacheFlags::AllowAsyncBuild) : EStreamedAudioCacheFlags::None;
		GetWorkingSoundWaveData()->RunningPlatformData.Cache(*this, CompressionOverrides, AudioFormat, CacheFlags, GetRunningPlatform());
	}
}

void USoundWave::BeginCachePlatformData()
{
	CachePlatformData(true);

#if WITH_EDITOR
	// enable caching in postload for derived data cache commandlet and cook by the book
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (TPM && (TPM->RestrictFormatsToRuntimeOnly() == false))
	{
		TArray<ITargetPlatform*> Platforms = TPM->GetActiveTargetPlatforms();
		// Cache for all the audio formats that the cooking target requires
		for (int32 FormatIndex = 0; FormatIndex < Platforms.Num(); FormatIndex++)
		{
			BeginCacheForCookedPlatformData(Platforms[FormatIndex]);
		}
	}
#endif
}
#if WITH_EDITOR

void USoundWave::BeginCacheForCookedPlatformData(const ITargetPlatform *TargetPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::BeginCacheForCookedPlatformData);

	const FPlatformAudioCookOverrides* CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides(*TargetPlatform->IniPlatformName());

	if (TargetPlatform->AllowAudioVisualData())
	{
		// Retrieve format to cache for targetplatform.
		FName PlatformFormat = TargetPlatform->GetWaveFormat(this);

		if (TargetPlatform->SupportsFeature(ETargetPlatformFeatures::AudioStreaming) && IsStreaming(*CompressionOverrides))
		{
			// Always allow the build to be performed asynchronously as it is now thread-safe by fetching compressed data directly from the DDC.
			uint32 CacheFlags = EStreamedAudioCacheFlags::Async | EStreamedAudioCacheFlags::InlineChunks | EStreamedAudioCacheFlags::AllowAsyncBuild;

			// find format data by comparing derived data keys.
			FString DerivedDataKey;
			GetStreamedAudioDerivedDataKeySuffix(*this, PlatformFormat, CompressionOverrides, TargetPlatform, DerivedDataKey);

			FStreamedAudioPlatformData *PlatformData = CookedPlatformData.FindRef(DerivedDataKey);

			if (PlatformData == nullptr)
			{
				PlatformData = new FStreamedAudioPlatformData();
				PlatformData->Cache(
					*this,
					CompressionOverrides,
					PlatformFormat,
					CacheFlags,
					TargetPlatform
					);
				CookedPlatformData.Add(DerivedDataKey, PlatformData);
			}
		}
		else
		{
			BeginGetCompressedData(PlatformFormat, CompressionOverrides, TargetPlatform);
		}
	}

	Super::BeginCacheForCookedPlatformData(TargetPlatform);
}

bool USoundWave::IsCachedCookedPlatformDataLoaded( const ITargetPlatform* TargetPlatform )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::IsCachedCookedPlatformDataLoaded);

	const FPlatformAudioCookOverrides* CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides(*TargetPlatform->IniPlatformName());

	if (TargetPlatform->AllowAudioVisualData())
	{
		// Retrieve format to cache for targetplatform.
		FName PlatformFormat = TargetPlatform->GetWaveFormat(this);

		if (TargetPlatform->SupportsFeature(ETargetPlatformFeatures::AudioStreaming) && IsStreaming(*CompressionOverrides))
		{
			// find format data by comparing derived data keys.
			FString DerivedDataKey;
			GetStreamedAudioDerivedDataKeySuffix(*this, PlatformFormat, CompressionOverrides, TargetPlatform, DerivedDataKey);

			FStreamedAudioPlatformData *PlatformData = CookedPlatformData.FindRef(DerivedDataKey);
			if (PlatformData == nullptr)
			{
				// we haven't called begincache
				return false;
			}

			if (PlatformData->IsAsyncWorkComplete())
			{
				PlatformData->FinishCache();
			}

			return PlatformData->IsFinishedCache();
		}
		else
		{
			return IsCompressedDataReady(PlatformFormat, CompressionOverrides);
		}
	}

	return true;
}


/**
* Clear all the cached cooked platform data which we have accumulated with BeginCacheForCookedPlatformData calls
* The data can still be cached again using BeginCacheForCookedPlatformData again
*/
void USoundWave::ClearAllCachedCookedPlatformData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::ClearAllCachedCookedPlatformData);

	Super::ClearAllCachedCookedPlatformData();

	for (auto It : CookedPlatformData)
	{
		delete It.Value;
	}

	CookedPlatformData.Empty();
}

void USoundWave::ClearCachedCookedPlatformData( const ITargetPlatform* TargetPlatform )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::ClearCachedCookedPlatformData);

	Super::ClearCachedCookedPlatformData(TargetPlatform);

	const FPlatformAudioCookOverrides* CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides(*TargetPlatform->IniPlatformName());

	if (TargetPlatform->SupportsFeature(ETargetPlatformFeatures::AudioStreaming) && IsStreaming(*CompressionOverrides))
	{
		// Retrieve format to cache for targetplatform.
		FName PlatformFormat = TargetPlatform->GetWaveFormat(this);

		// find format data by comparing derived data keys.
		FString DerivedDataKey;
		GetStreamedAudioDerivedDataKeySuffix(*this, PlatformFormat, CompressionOverrides, TargetPlatform, DerivedDataKey);


		if ( CookedPlatformData.Contains(DerivedDataKey) )
		{
			FStreamedAudioPlatformData *PlatformData = CookedPlatformData.FindAndRemoveChecked( DerivedDataKey );
			delete PlatformData;
		}
	}
}

void USoundWave::WillNeverCacheCookedPlatformDataAgain()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::WillNeverCacheCookedPlatformDataAgain);

	FinishCachePlatformData();

	// this is called after we have finished caching the platform data but before we have saved the data
	// so need to keep the cached platform data around
	Super::WillNeverCacheCookedPlatformDataAgain();

	GetWorkingSoundWaveData()->CompressedFormatData.FlushData();
}
#endif

void USoundWave::FinishCachePlatformData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::FinishCachePlatformData);

	// Removed the call to CachePlatformData here since the role of FinishCachePlatformData
	// should only be to finish any outstanding task. The only place that was relying on
	// FinishCachePlatformData to also call CachePlatformData was USoundWave::PostLoad
	// which has been modified to call CachePlatformData instead.
	// Furthermore, this function is called in WillNeverCacheCookedPlatformDataAgain, which we
	// obviously don't want to start performing new work, just finish the outstanding one.

	// Make sure async requests are finished
	GetWorkingSoundWaveData()->RunningPlatformData.FinishCache();

#if DO_CHECK
	// If we're allowing cooked data to be loaded then the derived data key will not have been serialized, so won't match and that's fine
	if (!GAllowCookedDataInEditorBuilds && GetWorkingSoundWaveData()->RunningPlatformData.GetNumChunks())
	{
		FString DerivedDataKey;
		FName AudioFormat = GetWaveFormatForRunningPlatform(*this);
		const FPlatformAudioCookOverrides* CompressionOverrides = GetCookOverridesForRunningPlatform();
		GetStreamedAudioDerivedDataKey(*this, AudioFormat,CompressionOverrides, GetRunningPlatform(), DerivedDataKey);

		UE_CLOGF(GetWorkingSoundWaveData()->RunningPlatformData.DerivedDataKey != DerivedDataKey, LogAudio, Warning, "Audio was cooked with the DDC key %ls but should've had the DDC key %ls. the cook overrides/codec used may be incorrect.", *GetWorkingSoundWaveData()->RunningPlatformData.DerivedDataKey, *DerivedDataKey);
	}
#endif
}

void USoundWave::ForceRebuildPlatformData()
{
	const FPlatformAudioCookOverrides* CompressionOverrides = GetCookOverridesForRunningPlatform();

	GetWorkingSoundWaveData()->RunningPlatformData.Cache(
		*this,
		CompressionOverrides,
		GetWaveFormatForRunningPlatform(*this),
		EStreamedAudioCacheFlags::ForceRebuild,
		GetRunningPlatform()
		);
}
#endif //WITH_EDITORONLY_DATA
