// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioDevice.h"

#include "ActiveSoundUpdateInterface.h"
#include "AudioCompressionSettingsUtils.h"
#include "AudioDecompress.h"
#include "AudioEffect.h"
#include "AudioPluginUtilities.h"
#include "Audio/AudioDebug.h"
#include "Audio/AudioTraceUtil.h"
#include "GameFramework/GameUserSettings.h"
#include "GameFramework/WorldSettings.h"
#include "GeneralProjectSettings.h"
#include "HAL/FileManager.h"
#include "IAudioParameterTransmitter.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreStats.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/PackageName.h"
#include "PhysicsEngine/BodyInstance.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Sound/SoundEffectSubmix.h"
#include "Sound/SoundGroups.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundSourceBus.h"
#include "Sound/SoundSubmix.h"
#include "Stats/StatsTrace.h"
#include "UnrealEngine.h"
#include "UObject/UObjectIterator.h"
#include "StereoRendering.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Editor/EditorEngine.h"
#else
#include "UObject/Package.h"
#endif // WITH_EDITOR

static int32 AudioChannelCountCVar = 0;
FAutoConsoleVariableRef CVarSetAudioChannelCount(
	TEXT("au.SetAudioChannelCount"),
	AudioChannelCountCVar,
	TEXT("Changes the audio channel count. Max value is clamped to the MaxChannelCount the audio engine was initialize with.\n")
	TEXT("0: Disable, >0: Enable"),
	ECVF_Default);

static float AudioChannelCountScaleCVar = 1.0f;
FAutoConsoleVariableRef CVarSetAudioChannelScaleCount(
	TEXT("au.SetAudioChannelScaleCount"),
	AudioChannelCountScaleCVar,
	TEXT("Changes the audio channel count by percentage.\n"),
	ECVF_Default);

static int32 DisableStoppingVoicesCvar = 0;
FAutoConsoleVariableRef CVarDisableStoppingVoices(
	TEXT("au.DisableStoppingVoices"),
	DisableStoppingVoicesCvar,
	TEXT("Disables stopping voices feature.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 DisableAppVolumeCvar = 0;
FAutoConsoleVariableRef CVarDisableAppVolume(
	TEXT("au.DisableAppVolume"),
	DisableAppVolumeCvar,
	TEXT("Disables application volume when set to 1.\n")
	TEXT("0: App volume enabled, 1: App volume disabled"),
	ECVF_Default);

static int32 DisableAutomaticPrecacheCvar = 0;
FAutoConsoleVariableRef CVarDisableAutomaticPrecache(
	TEXT("au.DisableAutomaticPrecache"),
	DisableAutomaticPrecacheCvar,
	TEXT("When set to 1, this disables precaching on load or startup, it will only precache synchronously when playing.\n")
	TEXT("0: Use normal precaching logic, 1: disables all precaching except for synchronous calls."),
	ECVF_Default);

static int32 WaitForSoundWaveToLoadCvar = 1;
FAutoConsoleVariableRef CVarWaitForSoundWaveToLoad(
	TEXT("au.WaitForSoundWaveToLoad"),
	WaitForSoundWaveToLoadCvar,
	TEXT("When set to 1, we will refuse to play any sound unless the USoundWave has been loaded.\n")
	TEXT("0: Attempt to play back, 1: Wait for load."),
	ECVF_Default);

static int32 BakedAnalysisEnabledCVar = 1;
FAutoConsoleVariableRef CVarBakedAnalysisEnabledCVar(
	TEXT("au.BakedAnalysisEnabled"),
	BakedAnalysisEnabledCVar,
	TEXT("Enables or disables queries to baked analysis from audio component.\n"),
	ECVF_Default);

static int32 NumPrecacheFramesCvar = 0;
FAutoConsoleVariableRef CVarNumPrecacheFrames(
	TEXT("au.NumPrecacheFrames"),
	NumPrecacheFramesCvar,
	TEXT("When set to > 0, will use that value as the number of frames to precache audio buffers with.\n")
	TEXT("0: Use default value for precache frames, >0: Number of frames to precache."),
	ECVF_Default);

static int32 DisableLegacyReverb = 0;
FAutoConsoleVariableRef CVarDisableLegacyReverb(
	TEXT("au.DisableLegacyReverb"),
	DisableLegacyReverb,
	TEXT("Disables reverb on legacy audio backends.\n")
	TEXT("0: Enabled, 1: Disabled"),
	ECVF_Default);

static float SoundDistanceOptimizationLengthCVar = 1.0f;
FAutoConsoleVariableRef CVarSoundDistanceOptimizationLength(
	TEXT("au.SoundDistanceOptimizationLength"),
	SoundDistanceOptimizationLengthCVar,
	TEXT("The maximum duration a sound must be in order to be a candidate to be culled due to one-shot distance optimization.\n"),
	ECVF_Default);

static int32 EnableBinauralAudioForAllSpatialSoundsCVar = 0;
FAutoConsoleVariableRef CVarEnableBinauralAudioForAllSpatialSounds(
	TEXT("au.EnableBinauralAudioForAllSpatialSounds"),
	EnableBinauralAudioForAllSpatialSoundsCVar,
	TEXT("Toggles binaural audio rendering for all spatial sounds if binaural rendering is available.\n"),
	ECVF_Default);

static int32 DisableBinauralSpatializationCVar = 0;
FAutoConsoleVariableRef CVarDisableBinauralSpatialization(
	TEXT("au.DisableBinauralSpatialization"),
	DisableBinauralSpatializationCVar,
	TEXT("Disables binaural spatialization.\n"),
	ECVF_Default);

static int32 FlushAudioRenderThreadOnGCCVar = 0;
FAutoConsoleVariableRef CVarFlushAudioRenderThreadOnGC(
	TEXT("au.FlushAudioRenderThreadOnGC"),
	FlushAudioRenderThreadOnGCCVar,
	TEXT("When set to 1, every time the GC runs, we flush all pending audio render thread commands.\n"),
	ECVF_Default);

static float MaxWorldDistanceCVar = UE_OLD_WORLD_MAX;
FAutoConsoleVariableRef CVarSetAudioMaxDistance(
	TEXT("au.MaxWorldDistance"),
	MaxWorldDistanceCVar,
	TEXT("Maximum world distance used in audio-related calculations (eg. attenuation).\n"),
	ECVF_Default);

static FAutoConsoleCommandWithWorld GListAvailableSpatialPluginsCommand(
	TEXT("au.spatialization.ListAvailableSpatialPlugins"),
	TEXT("This will output a list of currently available/active spatialization plugins"),
	FConsoleCommandWithWorldDelegate::CreateStatic(
		[](UWorld* InWorld)
	{
			if(InWorld)
			{
				if (FAudioDeviceHandle AudioDevice = InWorld->GetAudioDevice())
				{
					const TArray<FName> PluginNames = AudioDevice->GetAvailableSpatializationPluginNames();
					FName ActivePluginName = AudioDevice->GetCurrentSpatializationPluginInterfaceInfo().PluginName;

					if(GEngine)
					{
						for (FName PluginName : PluginNames)
						{
							const bool bIsActive = PluginName == ActivePluginName;
							GEngine->AddOnScreenDebugMessage(-1, 15.0f, FColor::Yellow
								, PluginName.ToString() + (bIsActive? '*' : ' '));
						}
						GEngine->AddOnScreenDebugMessage(-1, 15.0f, FColor::Yellow
							, FString::Printf(TEXT("-----------------------------------------")));
						GEngine->AddOnScreenDebugMessage(-1, 15.0f, FColor::Yellow
							, FString::Printf(TEXT("Available spatial plugins [count = %i] (* = Active)"), PluginNames.Num()));
					}
				}
			}
	})
);

static FAutoConsoleCommandWithWorldArgsAndOutputDevice GSetCurrentSpatialPluginCommand(
	TEXT("au.spatialization.SetCurrentSpatialPlugin"),
	TEXT("Attempt to swap to the named spatialization plugin (au.spatialization.ListAvailableSpatialPlugins to see what is availible)"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& PluginNameArray, UWorld* InWorld, FOutputDevice&)
		{
			FString PluginName;
			for(const FString& Substring : PluginNameArray)
			{
				PluginName += Substring;
				PluginName += ' ';
			}
			PluginName.TrimStartAndEndInline();

			if(InWorld)
			{
				if (FAudioDeviceHandle AudioDevice = InWorld->GetAudioDevice())
				{
					AudioDevice->SetCurrentSpatializationPlugin(FName(PluginName));
				}
			}
		})
	);


static int32 EnableRelativeRenderCostVoiceLimitCVar = 1;
FAutoConsoleVariableRef CVarEnableRelativeRenderCostVoiceLimit(
	TEXT("au.EnableRelativeRenderCostVoiceLimit"),
	EnableRelativeRenderCostVoiceLimitCVar,
	TEXT("Enables or disables using the relative render cost of rendering sources to count toward max channels culling."),
	ECVF_Default);

#if UE_AUDIO_PROFILERTRACE_ENABLED
UE_TRACE_EVENT_BEGIN(Audio, SoundStart)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AssetPath)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, EventLogName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SoundClassName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorLabel)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorIconName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AudioComponentName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundIsAlivePing)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AssetPath)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SoundClassName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorLabel)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AudioComponentName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundStop)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AssetPath)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, EventLogName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SoundClassName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorLabel)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorIconName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AudioComponentName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, VirtualLoopStopOrRealize)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
UE_TRACE_EVENT_END()

#endif // UE_AUDIO_PROFILERTRACE_ENABLED

namespace Audio
{
	ICompressedAudioInfo* CreateSoundAssetDecoder(const FName& InRuntimeFormat)
	{
		// The decoder will need to be instantiated for the specific platform
		if (InRuntimeFormat == Audio::NAME_PLATFORM_SPECIFIC)
		{
			return nullptr;
		}

		ICompressedAudioInfo* Decoder = IAudioInfoFactoryRegistry::Get().Create(InRuntimeFormat);
		ensureMsgf(Decoder, TEXT("Unknown runtime sound asset format type. '%s' "), *InRuntimeFormat.ToString());
		return Decoder;
	}

	uint64 GetTransmitterID(uint64 ComponentID, UPTRINT WaveInstanceHash, uint32 PlayOrder)
	{
		return HashCombineFast(static_cast<uint32>(ComponentID % TNumericLimits<uint32>::Max()), PlayOrder + WaveInstanceHash);
	}
}

namespace AudioDeviceUtils
{
	using FVirtualLoopPair = TPair<FActiveSound*, FAudioVirtualLoop>;

#if !UE_BUILD_SHIPPING
	int32 PrecachedRealtime = 0;
	int32 PrecachedNative = 0;
	int32 TotalNativeSize = 0;
	float AverageNativeLength = 0.f;
	TMap<int32, int32> NativeChannelCount;
	TMap<int32, int32> NativeSampleRateCount;
#endif // !UE_BUILD_SHIPPING
} // namespace <>

FAttenuationListenerData FAttenuationListenerData::Create(const FAudioDevice& AudioDevice, int32 ListenerIndex, const FTransform& InSoundTransform, const FSoundAttenuationSettings& InAttenuationSettings)
{
	FTransform ListenerTransform;
	AudioDevice.GetListenerTransform(ListenerIndex, ListenerTransform);

	FAttenuationListenerData ListenerData(ListenerTransform, InSoundTransform, InAttenuationSettings);

	const FVector SoundTranslation = InSoundTransform.GetTranslation();
	FVector ListenerToSound = SoundTranslation - ListenerTransform.GetTranslation();
	ListenerToSound.ToDirectionAndLength(ListenerData.ListenerToSoundDir, ListenerData.ListenerToSoundDistance);

	// Store the actual distance for surround-panning sources with spread (AudioMixer)
	ListenerData.ListenerToSoundDistanceForPanning = ListenerData.ListenerToSoundDistance;

	// Calculating override listener-to-sound distance and transform must
	// be applied after distance used for panning value is calculated.
	FVector ListenerPosition;
	const bool bAllowAttenuationOverride = true;
	if (AudioDevice.GetListenerPosition(ListenerIndex, ListenerPosition, bAllowAttenuationOverride))
	{
		ListenerData.ListenerToSoundDistance = (SoundTranslation - ListenerPosition).Size();
		ListenerData.ListenerTransform.SetTranslation(ListenerPosition);
	}

	const FSoundAttenuationSettings& AttenuationSettings = *ListenerData.AttenuationSettings;
	if ((AttenuationSettings.bAttenuate && AttenuationSettings.AttenuationShape == EAttenuationShape::Sphere) || AttenuationSettings.bAttenuateWithLPF)
	{
		ListenerData.AttenuationDistance = FMath::Max<float>(ListenerData.ListenerToSoundDistance - AttenuationSettings.AttenuationShapeExtents.X, 0.0f);
	}

	return MoveTemp(ListenerData);
}


/*-----------------------------------------------------------------------------
	FAudioDevice implementation.
-----------------------------------------------------------------------------*/

FAudioDevice::FAudioDevice()
	: NumStoppingSources(32)
	, SampleRate(0)
	, NumPrecacheFrames(MONO_PCM_BUFFER_SAMPLES)
	, DeviceID(static_cast<Audio::FDeviceId>(INDEX_NONE))
	, SourceDataOverridePluginInterface(nullptr)
	, ReverbPluginInterface(nullptr)
	, OcclusionInterface(nullptr)
	, MaxSources(0)
	, MaxChannels(0)
	, MaxChannels_GameThread(0)
	, MaxChannelsScale(1.0f)
	, MaxChannelsScale_GameThread(1.0f)
	, CurrentTick(0)
	, TestAudioComponent(nullptr)
	, DebugState(DEBUGSTATE_None)
	, TransientPrimaryVolume(1.0f)
	, PrimaryVolume(1.0f)
	, GlobalPitchScale(1.0f)
	, LastUpdateTime(FPlatformTime::Seconds())
	, NextResourceID(1)
	, BaseSoundMix(nullptr)
	, DefaultBaseSoundMix(nullptr)
	, Effects(nullptr)
	, CurrentReverbEffect(nullptr)
	, PlatformAudioHeadroom(1.0f)
	, DefaultReverbSendLevel(0.0f)
	, bHRTFEnabledForAll_OnGameThread(false)
	, bHRTFDisabled_OnGameThread(false)
	, bGameWasTicking(true)
	, bIsAudioDeviceHardwareInitialized(false)
	, bIsStoppingVoicesEnabled(false)
	, bIsBakedAnalysisEnabled(false)
	, bAudioMixerModuleLoaded(false)
	, bOcclusionIsExternalSend(false)
	, bReverbIsExternalSend(false)
	, bStartupSoundsPreCached(false)
	, bSpatializationInterfaceEnabled(false)
	, bOcclusionInterfaceEnabled(false)
	, bReverbInterfaceEnabled(false)
	, bSourceDataOverrideInterfaceEnabled(false)
	, bModulationInterfaceEnabled(false)
	, bPluginListenersInitialized(false)
	, bHRTFEnabledForAll(false)
	, bHRTFDisabled(false)
	, bIsDeviceMuted(false)
	, bIsInitialized(false)
	, AudioClock(0.0)
	, bAllowCenterChannel3DPanning(false)
	, DeviceDeltaTime(0.0f)
	, bHasActivatedReverb(false)
	, bAllowPlayWhenSilent(true)
	, bUseAttenuationForNonGameWorlds(false)
	, ConcurrencyManager(this)
	, OneShotCount(0)
	, GlobalMinPitch(0.4f)
	, GlobalMaxPitch(2.0f)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // suppress deprecation warning in default dtor
FAudioDevice::~FAudioDevice() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FAudioDeviceHandle FAudioDevice::GetMainAudioDevice()
{
	// Try to get GEngine's main audio device
	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();

	// If we don't have a main audio device (maybe we're running in a non-standard mode like a commandlet)
	if (!AudioDevice)
	{
		// We should have an active device for device manager
		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		return DeviceManager->GetActiveAudioDevice();
	}
	return AudioDevice;
}

FAudioDeviceManager* FAudioDevice::GetAudioDeviceManager()
{
	return GEngine->GetAudioDeviceManager();
}

FAudioEffectsManager* FAudioDevice::CreateEffectsManager()
{
	return new FAudioEffectsManager(this);
}

const FAudioQualitySettings& FAudioDevice::GetQualityLevelSettings()
{
	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	const int32 QualityLevel = GEngine ? GEngine->GetGameUserSettings()->GetAudioQualityLevel() : 0;
	return AudioSettings->GetQualityLevelSettings(QualityLevel);
}

bool FAudioDevice::Init(Audio::FDeviceId InDeviceID, int32 InMaxSources, int32 InBufferSizeOverride, int32 InNumBuffersOverride)
{
	SCOPED_BOOT_TIMING("FAudioDevice::Init");

	LLM_SCOPE(ELLMTag::AudioMisc);

	if (bIsInitialized)
	{
		return true;
	}

	if (InDeviceID == INDEX_NONE)
	{
		return false;
	}

	DeviceID = InDeviceID;

	bool bDeferStartupPrecache = false;

	PluginListeners.Reset();

	// Initialize MaxChannels taking into account platform configurations
	// Get a copy of the platform-specific settings (overridden by platforms)
	PlatformSettings = GetPlatformSettings();

	// Override platform settings with the command line
	if (FParse::Value(FCommandLine::Get(), TEXT("-AudioCallbackBufferFrameSize="), PlatformSettings.CallbackBufferFrameSize))
	{
		UE_LOGF(LogAudioMixer, Display, "Command Line CallbackBufferFrameSize Override: %d", PlatformSettings.CallbackBufferFrameSize);
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-AudioNumBuffersToEnqueue="), PlatformSettings.NumBuffers))
	{
		UE_LOGF(LogAudioMixer, Display, "Command Line NumBuffersToEnqueue Override: %d", PlatformSettings.NumBuffers);
	}

	// Override arguments passed into this function trump command line argument overrides
	if (InBufferSizeOverride != INDEX_NONE)
	{
		PlatformSettings.CallbackBufferFrameSize = InBufferSizeOverride;
	}
	if (InNumBuffersOverride != INDEX_NONE)
	{
		PlatformSettings.NumBuffers = InNumBuffersOverride;
	}

	// Validate buffer size and num buffers
 	PlatformSettings.CallbackBufferFrameSize = FMath::Max(4, PlatformSettings.CallbackBufferFrameSize);
 	PlatformSettings.CallbackBufferFrameSize = PlatformSettings.CallbackBufferFrameSize - (PlatformSettings.CallbackBufferFrameSize % 4);
 	PlatformSettings.NumBuffers = FMath::Max(1, PlatformSettings.NumBuffers);

	// MaxSources is the max value supplied to Init call (quality settings), unless overwritten by the platform settings.
	// This does not have to be the minimum value in this case (nor is it desired, so platforms can potentially scale up)
	// as the Sources array has yet to be initialized. If the cvar is largest, take that value to allow for testing
	const int32 PlatformMaxSources = PlatformSettings.MaxChannels > 0 ? PlatformSettings.MaxChannels : InMaxSources;
	MaxSources = FMath::Max(PlatformMaxSources, AudioChannelCountCVar);
	MaxSources = FMath::Max(MaxSources, 1);
	UE_LOGF(LogAudio, Display, "AudioDevice MaxSources: %d", MaxSources);

	MaxChannels = MaxSources;
	MaxChannels_GameThread = MaxSources;

	// Mixed sample rate is set by the platform
	SampleRate = PlatformSettings.SampleRate;

	// If this is true, skip the initial startup precache so we can do it later in the flow
	GConfig->GetBool(TEXT("Audio"), TEXT("DeferStartupPrecache"), bDeferStartupPrecache, GEngineIni);

	// Get an optional engine ini setting for platform headroom.
	float Headroom = 0.0f; // in dB
	if (GConfig->GetFloat(TEXT("Audio"), TEXT("PlatformHeadroomDB"), Headroom, GEngineIni))
	{
		// Convert dB to linear volume
		PlatformAudioHeadroom = FMath::Pow(10.0f, Headroom / 20.0f);
	}

	int32 NumPrecacheFramesSettings = 0;
	if (GConfig->GetInt(TEXT("Audio"), TEXT("NumPrecacheFrames"), NumPrecacheFramesSettings, GEngineIni))
	{
		NumPrecacheFrames = FMath::Min(128, NumPrecacheFramesSettings);
	}

	bIsStoppingVoicesEnabled = !DisableStoppingVoicesCvar;

	bIsBakedAnalysisEnabled = (BakedAnalysisEnabledCVar == 1);

	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();

	GlobalMinPitch = FMath::Max(AudioSettings->GlobalMinPitchScale, 0.0001f);
	GlobalMaxPitch = FMath::Max(AudioSettings->GlobalMaxPitchScale, 0.0001f);
	bAllowCenterChannel3DPanning = AudioSettings->bAllowCenterChannel3DPanning;
	bAllowPlayWhenSilent = AudioSettings->bAllowPlayWhenSilent;
	DefaultReverbSendLevel = AudioSettings->DefaultReverbSendLevel_DEPRECATED;

	const FSoftObjectPath DefaultBaseSoundMixName = GetDefault<UAudioSettings>()->DefaultBaseSoundMix;
	if (DefaultBaseSoundMixName.IsValid())
	{
		DefaultBaseSoundMix = LoadObject<USoundMix>(nullptr, *DefaultBaseSoundMixName.ToString());
	}

	GetDefault<USoundGroups>()->Initialize();

	// Parses sound classes.
	InitSoundClasses();

	// Audio mixer needs to create effects manager before initializing the plugins.
	if (IsStoppingVoicesEnabled())
	{
		// create a platform specific effects manager
		Effects = CreateEffectsManager();

		NumStoppingSources = GetDefault<UAudioSettings>()->NumStoppingSources;
	}
	else
	{
		// Stopping sources are not supported in the old audio engine
		NumStoppingSources = 0;
	}

	{
		LLM_SCOPE(ELLMTag::AudioMixerPlugins);


	//Get the requested default spatialization plugin and set it up.
	IAudioSpatializationFactory* SpatializationPluginFactory = AudioPluginUtilities::GetDesiredSpatializationPlugin();
	if (SpatializationPluginFactory != nullptr)
	{
		// Cache info for the available spatial plugins
		TArray<IAudioSpatializationFactory*> SpatializerFactories = AudioPluginUtilities::GetSpatialPluginArray();
		for(const auto& Factory : SpatializerFactories)
		{
			if(!ensure(Factory))
			{
				continue;  // null factory ptr in the modular features array
			}

			const FName PluginName = FName(Factory->GetDisplayName());
			FAudioSpatializationInterfaceInfo SpatPluginInfo(PluginName, /* InAudioDevice */ this, Factory);

			SpatializationInterfaces.Add(SpatPluginInfo);
		}

		bSpatializationInterfaceEnabled = true;

		UE_LOGF(LogAudio, Display, "Audio Spatialization Plugin: %ls is external send: %d", *(SpatializationPluginFactory->GetDisplayName()), SpatializationPluginFactory->IsExternalSend());
	}
	else
	{
		UE_LOGF(LogAudio, Display, "Audio Spatialization Plugin: None (built-in).");
	}

	IAudioSourceDataOverrideFactory* SourceDataOverridePluginFactory = AudioPluginUtilities::GetDesiredSourceDataOverridePlugin();
	if (SourceDataOverridePluginFactory)
	{
		SourceDataOverridePluginInterface = SourceDataOverridePluginFactory->CreateNewSourceDataOverridePlugin(this);
		bSourceDataOverrideInterfaceEnabled = true;
		UE_LOGF(LogAudio, Display, "Audio Source Data Override Plugin: %ls", *(SourceDataOverridePluginFactory->GetDisplayName()));
	}

	//Get the requested reverb plugin and set it up:
	IAudioReverbFactory* ReverbPluginFactory = AudioPluginUtilities::GetDesiredReverbPlugin();
	if (ReverbPluginFactory != nullptr)
	{
		ReverbPluginInterface = ReverbPluginFactory->CreateNewReverbPlugin(this);
		bReverbInterfaceEnabled = true;
		bReverbIsExternalSend = ReverbPluginFactory->IsExternalSend();
		UE_LOGF(LogAudio, Display, "Audio Reverb Plugin: %ls", *(ReverbPluginFactory->GetDisplayName()));
	}
	else
	{
		UE_LOGF(LogAudio, Display, "Audio Reverb Plugin: None (built-in).");
	}

	//Get the requested occlusion plugin and set it up.
	IAudioOcclusionFactory* OcclusionPluginFactory = AudioPluginUtilities::GetDesiredOcclusionPlugin();
	if (OcclusionPluginFactory != nullptr)
	{
		OcclusionInterface = OcclusionPluginFactory->CreateNewOcclusionPlugin(this);
		bOcclusionInterfaceEnabled = true;
		bOcclusionIsExternalSend = OcclusionPluginFactory->IsExternalSend();
		UE_LOGF(LogAudio, Display, "Audio Occlusion Plugin: %ls", *(OcclusionPluginFactory->GetDisplayName()));
	}
	else
	{
		UE_LOGF(LogAudio, Display, "Audio Occlusion Plugin: None (built-in).");
	}

	//Get the requested modulation plugin and set it up.
	if (IAudioModulationFactory* ModulationPluginFactory = AudioPluginUtilities::GetDesiredModulationPlugin())
	{
		ModulationInterface = ModulationPluginFactory->CreateNewModulationPlugin(this);

		//Set up initialization parameters for system level effect plugins:
		PluginInitializationParams.SampleRate = SampleRate;
		PluginInitializationParams.NumSources = GetMaxSources();
		PluginInitializationParams.BufferLength = PlatformSettings.CallbackBufferFrameSize;
		PluginInitializationParams.AudioDevicePtr = this;
		ModulationInterface->Initialize(PluginInitializationParams);

		bModulationInterfaceEnabled = true;
		UE_LOGF(LogAudio, Display, "Audio Modulation Plugin: %ls", *(ModulationPluginFactory->GetDisplayName().ToString()));
	}
	}

	// allow the platform to startup
	if (!InitializeHardware())
	{
		// Could not initialize hardware. Tear down anything that was set up during initialization.
		UE_LOGF(LogAudio, Warning, "Could not initialize hardware. Tearing down anything that was set up during initialization");
		Teardown();

		return false;
	}

	InitSoundSources();

	// Make sure the Listeners array has at least one entry, so we don't have to check for Listeners.Num() == 0 all the time
	Listeners.Add(FListener(this));
	ListenerProxies.AddDefaulted();
	FMatrix Transform;
	Transform.SetIdentity();
	InverseListenerTransforms.Reset();
	InverseListenerTransforms.Add(Transform);

	if (!bDeferStartupPrecache)
	{
		PrecacheStartupSounds();
	}

	bIsInitialized = true;

	DeviceCreatedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddRaw(this, &FAudioDevice::OnDeviceCreated);
	DeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddRaw(this, &FAudioDevice::OnDeviceDestroyed);

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FAudioDevice::OnPreGarbageCollect);
	FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy.AddRaw(this, &FAudioDevice::OnPreGarbageCollect);

	UE_LOGF(LogInit, Log, "FAudioDevice initialized with ID %d.", InDeviceID);

	return true;
}

void FAudioDevice::OnDeviceCreated(Audio::FDeviceId InDeviceID)
{
	if (InDeviceID == DeviceID)
	{
		InitializeSubsystemCollection();
		FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Remove(DeviceCreatedHandle);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		FTraceAuxiliary::OnTraceStarted.AddRaw(this, &FAudioDevice::OnTraceStarted);
#endif
	}
}

void FAudioDevice::OnDeviceDestroyed(Audio::FDeviceId InDeviceID)
{
	if (InDeviceID == DeviceID)
	{
		FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		FTraceAuxiliary::OnTraceStarted.RemoveAll(this);
#endif
	}
}

void FAudioDevice::OnPreGarbageCollect()
{
	if (FlushAudioRenderThreadOnGCCVar)
	{
		FlushAudioRenderingCommands();
	}
}

float FAudioDevice::GetLowPassFilterResonance() const
{
	// hard-coded to the default value vs being store in the settings since this shouldn't be a global audio settings value
	return 0.9f;
}


bool FAudioDevice::SetCurrentSpatializationPlugin(FName PluginName)
{
	if(PluginName == CurrentSpatializationPluginInterfaceName)
	{
		return true; // no need to do anything.
	}

	StopAllSounds(false);

	for (FAudioSpatializationInterfaceInfo& Plugin : SpatializationInterfaces)
	{
		if(Plugin.PluginName == PluginName)
		{
			CurrentSpatializationPluginInterfaceName = PluginName;
			CurrentSpatializationInterfaceInfoPtr = &Plugin;

			if(!Plugin.bIsInitialized && Plugin.SpatializationPlugin)
			{
				Plugin.SpatializationPlugin->Initialize(PluginInitializationParams);
				Plugin.bIsInitialized = true;
			}

			return true;
		}
	}

	return false;
}

TArray<FName> FAudioDevice::GetAvailableSpatializationPluginNames() const
{
	TArray<FName> SpatPluginNames;
	for(const auto& Plugin : SpatializationInterfaces)
	{
		SpatPluginNames.Add(Plugin.PluginName);
	}

	return SpatPluginNames;
}

FAudioDevice::FAudioSpatializationInterfaceInfo FAudioDevice::GetCurrentSpatializationPluginInterfaceInfo()
{
	// See if we need to update the ptr to the current info
	if(!CurrentSpatializationInterfaceInfoPtr || CurrentSpatializationInterfaceInfoPtr->PluginName != CurrentSpatializationPluginInterfaceName)
	{
		CurrentSpatializationInterfaceInfoPtr = nullptr;
		for (FAudioSpatializationInterfaceInfo& Plugin : SpatializationInterfaces)
		{
			if(Plugin.PluginName == CurrentSpatializationPluginInterfaceName)
			{
				CurrentSpatializationInterfaceInfoPtr = &Plugin;
				return *CurrentSpatializationInterfaceInfoPtr;
			}
		}
	}

	// return the struct if we have it
	if(CurrentSpatializationInterfaceInfoPtr)
	{
		return *CurrentSpatializationInterfaceInfoPtr;
	}

	return {};
}

bool FAudioDevice::SpatializationPluginInterfacesAvailable()
{
	return GetCurrentSpatializationPluginInterfaceInfo().IsValid();
}

bool FAudioDevice::IsOcclusionPluginLoaded()
{
	if (FAudioDeviceHandle MainAudioDevice = GEngine->GetMainAudioDevice())
	{
		return MainAudioDevice->bOcclusionInterfaceEnabled;
	}
	return false;
}

bool FAudioDevice::IsReverbPluginLoaded()
{
	if (FAudioDeviceHandle MainAudioDevice = GEngine->GetMainAudioDevice())
	{
		return MainAudioDevice->bReverbInterfaceEnabled;
	}
	return false;
}

bool FAudioDevice::IsSourceDataOverridePluginLoaded()
{
	if (FAudioDeviceHandle MainAudioDevice = GEngine->GetMainAudioDevice())
	{
		return MainAudioDevice->bSourceDataOverrideInterfaceEnabled;
	}
	return false;
}

void FAudioDevice::PrecacheStartupSounds()
{
	// Iterate over all already loaded sounds and precache them. This relies on Super::Init in derived classes to be called last.
	if (!GIsEditor && GEngine && GEngine->UseSound() && DisableAutomaticPrecacheCvar == 0)
	{
		for (TObjectIterator<USoundWave> It; It; ++It)
		{
			USoundWave* SoundWave = *It;
			Precache(SoundWave, false, false, false);
		}

		bStartupSoundsPreCached = true;
	}
}

void FAudioDevice::SetMaxChannels(int32 InMaxChannels)
{
	if (InMaxChannels <= 0)
	{
		UE_LOGF(LogAudio, Warning, "MaxChannels must be set to a positive value.");
		return;
	}

	if (InMaxChannels > MaxSources)
	{
		UE_LOGF(LogAudio, Warning, "Can't increase MaxChannels past MaxSources");
		return;
	}

	if (IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetMaxChannelsGameThread"), STAT_AudioSetMaxChannelsGameThread, STATGROUP_AudioThreadCommands);
		FAudioThread::RunCommandOnGameThread([this, InMaxChannels]()
		{
			MaxChannels_GameThread = InMaxChannels;

		}, GET_STATID(STAT_AudioSetMaxChannelsGameThread));
	}

	if (IsInGameThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetMaxChannels"), STAT_AudioSetMaxChannels, STATGROUP_AudioThreadCommands);

		FAudioThread::RunCommandOnAudioThread([this, InMaxChannels]()
		{
			MaxChannels = InMaxChannels;
		}, GET_STATID(STAT_AudioSetMaxChannels));
	}
}

void FAudioDevice::SetMaxChannelsScaled(float InScaledChannelCount)
{
	if (!IsInAudioThread())
	{
		MaxChannelsScale_GameThread = InScaledChannelCount;

		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetMaxChannelsScaled"), STAT_AudioSetMaxChannelsScaled, STATGROUP_AudioThreadCommands);

		FAudioThread::RunCommandOnAudioThread([this, InScaledChannelCount]()
		{
			MaxChannelsScale = FMath::Clamp(InScaledChannelCount, 0.0f, 1.0f);

		}, GET_STATID(STAT_AudioSetMaxChannelsScaled));

		return;
	}
	else
	{
		MaxChannelsScale = FMath::Clamp(InScaledChannelCount, 0.0f, 1.0f);

		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetMaxChannelsScaled"), STAT_AudioSetMaxChannelsScaled, STATGROUP_AudioThreadCommands);

		FAudioThread::RunCommandOnGameThread([this, InScaledChannelCount]()
		{
			MaxChannelsScale_GameThread = InScaledChannelCount;

		}, GET_STATID(STAT_AudioSetMaxChannelsScaled));
	}
}

int32 FAudioDevice::GetMaxChannels() const
{
	// Get thread-context version of channel scalar & scale by cvar scalar
	float MaxChannelScalarToApply = IsInAudioThread() ? MaxChannelsScale : MaxChannelsScale_GameThread;
	MaxChannelScalarToApply *= AudioChannelCountScaleCVar;

	// Get thread-context version of channel max.  Override by cvar if cvar is valid.
	int32 OutMaxChannels = IsInAudioThread() ? MaxChannels : MaxChannels_GameThread;
	if (AudioChannelCountCVar > 0)
	{
		OutMaxChannels = AudioChannelCountCVar;
	}

	// Find product of max channels and final scalar, and clamp between 1 and MaxSources.
	check(MaxSources > 0);
	return FMath::Clamp(static_cast<int32>(OutMaxChannels * MaxChannelScalarToApply), 1, MaxSources);
}

int32 FAudioDevice::GetMaxSources() const
{
	return MaxSources + NumStoppingSources;
}

TRange<float> FAudioDevice::GetGlobalPitchRange() const
{
	return TRange<float>(GlobalMinPitch, GlobalMaxPitch);
}

void FAudioDevice::Teardown()
{
#if !UE_BUILD_SHIPPING
	const bool bDidRemoveDelegate = FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Remove(DeviceCreatedHandle);
	checkf(!bDidRemoveDelegate, TEXT("DelegateHandle not removed. Should be removed during FAudioDevice::OnDeviceDestroyed(...)"));
#endif

	// Make sure we process any pending game thread tasks before tearing down the audio device.
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

	// Do a fadeout to prevent clicking on shutdown
	FadeOut();

	// Flush stops all sources so sources can be safely deleted below.
	Flush(nullptr);

	// Clear out the EQ/Reverb/LPF effects
	if (Effects)
	{
		delete Effects;
		Effects = nullptr;
	}

	for (TAudioPluginListenerPtr PluginListener : PluginListeners)
	{
		AUDIO_SPATIALIZATION_PLUGIN_LLM_SCOPE

		PluginListener->OnListenerShutdown(this);
	}


	// let platform shutdown
	TeardownHardware();

	SoundMixClassEffectOverrides.Empty();

	// Note: we don't free audio buffers at this stage since they are managed in the audio device manager

	// Must be after FreeBufferResource as that potentially stops sources
	for (int32 Index = 0; Index < Sources.Num(); Index++)
	{
		// Make the sound stop instantly
		Sources[Index]->StopNow();
		delete Sources[Index];
	}

	Sources.Reset();
	FreeSources.Reset();

	LLM_SCOPE(ELLMTag::AudioMixerPlugins);

	// shutdown our active audio plugins

	// (spatial)
	for(const auto& Plugin : SpatializationInterfaces)
	{
		if(TAudioSpatializationPtr SpatialPluginPtr = Plugin.SpatializationPlugin)
		{
			SpatialPluginPtr->Shutdown();
		}
	}

	SpatializationInterfaces.Empty();
	bSpatializationInterfaceEnabled = false;

	// (reverb)
	if (ReverbPluginInterface.IsValid())
	{
		ReverbPluginInterface->Shutdown();
		ReverbPluginInterface.Reset();
		bReverbInterfaceEnabled = false;
	}

	// (source data override)
	if (SourceDataOverridePluginInterface.IsValid())
	{
		SourceDataOverridePluginInterface.Reset();
		bSourceDataOverrideInterfaceEnabled = false;
	}

	// (Occlusion)
	if (OcclusionInterface.IsValid())
	{
		OcclusionInterface->Shutdown();
		OcclusionInterface.Reset();
		bOcclusionInterfaceEnabled = false;
	}

	// (modulation)
	ModulationInterface.Reset();
	bModulationInterfaceEnabled = false;

	PluginListeners.Reset();

#if ENABLE_AUDIO_DEBUG
	Audio::FAudioDebugger::RemoveDevice(*this);
#endif // ENABLE_AUDIO_DEBUG

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
	FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy.RemoveAll(this);
}

void FAudioDevice::Deinitialize()
{
	SubsystemCollection.Deinitialize();
	SubsystemCollectionRoot.Reset();
}

void FAudioDevice::Suspend(bool bGameTicking)
{
	HandlePause(bGameTicking, true);
}

void FAudioDevice::CountBytes(FArchive& Ar)
{
	Sources.CountBytes(Ar);
	// The buffers are stored on the audio device since they are shared amongst all audio devices
	// Though we are going to count them when querying an individual audio device object about its bytes
	FreeSources.CountBytes(Ar);
	WaveInstanceSourceMap.CountBytes(Ar);
	Ar.CountBytes(sizeof(FWaveInstance) * WaveInstanceSourceMap.Num(), sizeof(FWaveInstance) * WaveInstanceSourceMap.Num());
	SoundClasses.CountBytes(Ar);
	SoundMixModifiers.CountBytes(Ar);
}


void FAudioDevice::AddReferencedObjects(FReferenceCollector& Collector)
{
	/** Threading Note: The FAudioThread may run concurrently with `AddReferencedObjects(...). 
	 *
	 * The FAudioThread is suspended during GC to avoid accesing objects during
	 * or after their destruction. But the FAudioThread does NOT suspend during 
	 * AddReferencedObjects(...). Any object that needs to be accessed on the FAudioThread
	 * but needs protection from GC should be adding there references to the reference collector. 
	 * 
	 * In FAudioDevice, there are two primary strategies used to achieve this, where systems
	 * generally choose one strategy depending on their system's needs, though sometimes require
	 * both strategies in more complex scenarios. 
	 *
	 * Strategy 1: Manage the lifetime of the object entirely on the GT, where GT logic can
	 * ensure that the object will be protected from GC any time the object is accessed
	 * on the FAudioThread.  
	 *
	 * Strategy 2: Manage the lifetime of the object on the AT and utilize the AudioThreadReferenceCritSec
	 * to protect against modifying the object while it is being read during `AddReferencedObjects(...)`. 
	 */

	Collector.AddReferencedObject(DefaultBaseSoundMix);
	Collector.AddReferencedObjects(PrevPassiveSoundMixModifiers);
	Collector.AddReferencedObjects(SoundMixModifiers);

	for (TPair<FName, FActivatedReverb>& ActivatedReverbPair : ActivatedReverbs)
	{
		Collector.AddPropertyReferencesWithStructARO(FReverbSettings::StaticStruct(), &ActivatedReverbPair.Value.ReverbSettings);
	}

	if (PendingActivatedReverb)
	{
		Collector.AddPropertyReferencesWithStructARO(FReverbSettings::StaticStruct(), &PendingActivatedReverb->ReverbSettings);
	}

	if (Effects)
	{
		Effects->AddReferencedObjects(Collector);
	}

	for (FActiveSound* ActiveSound : ActiveSounds)
	{
		ActiveSound->AddReferencedObjects(Collector);
	}

	for (FActiveSound* ActiveSound : PendingSoundsToDelete)
	{
		ActiveSound->AddReferencedObjects(Collector);
	}

	for (AudioDeviceUtils::FVirtualLoopPair& Pair : VirtualLoops)
	{
		Pair.Key->AddReferencedObjects(Collector);
	}

	// Make sure our referenced sound waves are up-to-date
	UpdateReferencedSoundWaves();

	// Make sure we don't try to delete any sound waves which may have in-flight decodes
	Collector.AddReferencedObjects(ReferencedSoundWaves);

	{
		// Lock to protect against modification of objects which are owned by the AudioThread, but need
		// their references tracked to protect against GC. 
		FScopeLock AudioThreadReferenceLock(&AudioThreadReferenceCritSec);

		Collector.AddPropertyReferencesWithStructARO(FReverbSettings::StaticStruct(), &HighestPriorityActivatedReverb_AudioThread.ReverbSettings);

		for (FAudioVolumeSubmixSendSettings& Settings : CurrentAudioVolumeSettings.SubmixSendSettings)
		{
			Collector.AddPropertyReferencesWithStructARO(FAudioVolumeSubmixSendSettings::StaticStruct(), &Settings);
		}
		for (FAudioVolumeSubmixOverrideSettings& Settings : CurrentAudioVolumeSettings.SubmixOverrideSettings)
		{
			Collector.AddPropertyReferencesWithStructARO(FAudioVolumeSubmixOverrideSettings::StaticStruct(), &Settings);
		}
	}
}

void FAudioDevice::ResetInterpolation()
{
	check(IsInAudioThread());

	for (FListener& Listener : Listeners)
	{
		Listener.InteriorStartTime = 0.f;
		Listener.InteriorEndTime = 0.f;
		Listener.ExteriorEndTime = 0.f;
		Listener.InteriorLPFEndTime = 0.f;
		Listener.ExteriorLPFEndTime = 0.f;

		Listener.InteriorVolumeInterp = 0.0f;
		Listener.InteriorLPFInterp = 0.0f;
		Listener.ExteriorVolumeInterp = 0.0f;
		Listener.ExteriorLPFInterp = 0.0f;
	}

	// Reset sound class properties to defaults
	for (TMap<USoundClass*, FSoundClassProperties>::TIterator It(SoundClasses); It; ++It)
	{
		USoundClass* SoundClass = It.Key();
		if (SoundClass)
		{
			It.Value() = SoundClass->Properties;
		}
	}

	SoundMixModifiers.Reset();
	PrevPassiveSoundMixModifiers.Reset();
	BaseSoundMix = nullptr;

	// reset audio effects
	if (Effects)
	{
		Effects->ResetInterpolation();
	}
}

void FAudioDevice::EnableRadioEffect(bool bEnable)
{
	if (bEnable)
	{
		SetMixDebugState(DEBUGSTATE_None);
	}
	else
	{
		UE_LOGF(LogAudio, Log, "Radio disabled for all sources");
		SetMixDebugState(DEBUGSTATE_DisableRadio);
	}
}

#if !UE_BUILD_SHIPPING
UAudioComponent* FAudioDevice::GetTestComponent(UWorld* InWorld)
{
	if (InWorld)
	{
		if (!TestAudioComponent.IsValid() || TestAudioComponent->GetWorld() != InWorld)
		{
			TestAudioComponent = TStrongObjectPtr<UAudioComponent>(NewObject<UAudioComponent>());
			TestAudioComponent->RegisterComponentWithWorld(InWorld);
		}

		return TestAudioComponent.Get();
	}

	return nullptr;
}

void FAudioDevice::StopTestComponent()
{
	if (TestAudioComponent.IsValid())
	{
		TestAudioComponent->Stop();
	}
}

bool FAudioDevice::HandleShowSoundClassHierarchyCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioThreadSuspendContext AudioThreadSuspend;

	ShowSoundClassHierarchy(Ar);
	return true;
}

bool FAudioDevice::HandleListWavesCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioThreadSuspendContext AudioThreadSuspend;

	TArray<FWaveInstance*> WaveInstances;
	int32 FirstActiveIndex = GetSortedActiveWaveInstances(WaveInstances, ESortedActiveWaveGetType::QueryOnly);

	for (int32 InstanceIndex = FirstActiveIndex; InstanceIndex < WaveInstances.Num(); InstanceIndex++)
	{
		FWaveInstance* WaveInstance = WaveInstances[ InstanceIndex ];
		FSoundSource* Source = WaveInstanceSourceMap.FindRef(WaveInstance);
		UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(WaveInstance->ActiveSound->GetAudioComponentID());
		AActor* SoundOwner = AudioComponent ? AudioComponent->GetOwner() : nullptr;
		Ar.Logf(TEXT("%4i.    %s %6.2f %6.2f  %s   %s"), InstanceIndex, Source ? TEXT("Yes") : TEXT(" No"), WaveInstance->ActiveSound->PlaybackTime, WaveInstance->GetVolume(), *WaveInstance->WaveData->GetPathName(), SoundOwner ? *SoundOwner->GetName() : TEXT("None"));
	}

	Ar.Logf(TEXT("Total: %i"), WaveInstances.Num()-FirstActiveIndex);

	return true;
}

void FAudioDevice::GetSoundClassInfo(TMap<FName, FAudioClassInfo>& AudioClassInfos)
{
	// Iterate over all sound cues to get a unique map of sound node waves to class names
	TMap<USoundWave*, FName> SoundWaveClasses;

	for (TObjectIterator<USoundCue> CueIt; CueIt; ++CueIt)
	{
		TArray<USoundNodeWavePlayer*> WavePlayers;

		USoundCue* SoundCue = *CueIt;
		SoundCue->RecursiveFindNode<USoundNodeWavePlayer>(SoundCue->FirstNode, WavePlayers);

		for (int32 WaveIndex = 0; WaveIndex < WavePlayers.Num(); ++WaveIndex)
		{
			// Presume one class per sound node wave
			USoundWave *SoundWave = WavePlayers[ WaveIndex ]->GetSoundWave();
			if (SoundWave && SoundCue->GetSoundClass())
			{
				SoundWaveClasses.Add(SoundWave, SoundCue->GetSoundClass()->GetFName());
			}
		}
	}

	// Add any sound node waves that are not referenced by sound cues
	for (TObjectIterator<USoundWave> WaveIt; WaveIt; ++WaveIt)
	{
		USoundWave* SoundWave = *WaveIt;
		if (SoundWaveClasses.Find(SoundWave) == nullptr)
		{
			SoundWaveClasses.Add(SoundWave, NAME_UnGrouped);
		}
	}

	// Collate the data into something useful
	for (TMap<USoundWave*, FName>::TIterator MapIter(SoundWaveClasses); MapIter; ++MapIter)
	{
		USoundWave* SoundWave = MapIter.Key();
		FName ClassName = MapIter.Value();

		FAudioClassInfo* AudioClassInfo = AudioClassInfos.Find(ClassName);
		if (AudioClassInfo == nullptr)
		{
			FAudioClassInfo NewAudioClassInfo;

			NewAudioClassInfo.NumResident = 0;
			NewAudioClassInfo.SizeResident = 0;
			NewAudioClassInfo.NumRealTime = 0;
			NewAudioClassInfo.SizeRealTime = 0;

			AudioClassInfos.Add(ClassName, NewAudioClassInfo);

			AudioClassInfo = AudioClassInfos.Find(ClassName);
			check(AudioClassInfo);
		}

#if !WITH_EDITOR
		AudioClassInfo->SizeResident += SoundWave->GetCompressedDataSize(SoundWave->GetRuntimeFormat());
		AudioClassInfo->NumResident++;
#else
		switch(SoundWave->DecompressionType)
		{
		case DTYPE_RealTime:
			AudioClassInfo->SizeRealTime += SoundWave->GetCompressedDataSize(SoundWave->GetRuntimeFormat());
			AudioClassInfo->NumRealTime++;
			break;

		case DTYPE_Streaming:
			// Add these to real time count for now - eventually compressed data won't be loaded &
			// might have a class info entry of their own
			AudioClassInfo->SizeRealTime += SoundWave->GetCompressedDataSize(SoundWave->GetRuntimeFormat());
			AudioClassInfo->NumRealTime++;
			break;

		case DTYPE_Setup:
		case DTYPE_Invalid:
		default:
			break;
		}
#endif
	}
}

bool FAudioDevice::HandleListSoundClassesCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	TMap<FName, FAudioClassInfo> AudioClassInfos;

	GetSoundClassInfo(AudioClassInfos);

	Ar.Logf(TEXT("Listing all sound classes."));

	// Display the collated data
	int32 TotalSounds = 0;
	for (TMap<FName, FAudioClassInfo>::TIterator ACIIter(AudioClassInfos); ACIIter; ++ACIIter)
	{
		FName ClassName = ACIIter.Key();
		FAudioClassInfo* ACI = AudioClassInfos.Find(ClassName);

		FString Line = FString::Printf(TEXT("Class '%s' has %d resident sounds taking %.2f kb"), *ClassName.ToString(), ACI->NumResident, ACI->SizeResident / 1024.0f);
		TotalSounds += ACI->NumResident;
		if (ACI->NumRealTime > 0)
		{
			Line += FString::Printf(TEXT(", and %d real time sounds taking %.2f kb "), ACI->NumRealTime, ACI->SizeRealTime / 1024.0f);
			TotalSounds += ACI->NumRealTime;
		}

		Ar.Logf(TEXT("%s"), *Line);
	}

	Ar.Logf(TEXT("%d total sounds in %d classes"), TotalSounds, AudioClassInfos.Num());
	return true;
}

void FAudioDevice::ShowSoundClassHierarchy(FOutputDevice& Ar, USoundClass* InSoundClass, int32 Indent ) const
{
	TArray<USoundClass*> SoundClassesToShow;
	if (InSoundClass)
	{
		SoundClassesToShow.Add(InSoundClass);
	}
	else
	{
		for (TMap<USoundClass*, FSoundClassProperties>::TConstIterator It(SoundClasses); It; ++It)
		{
			USoundClass* SoundClass = It.Key();
			if (SoundClass && SoundClass->ParentClass == nullptr)
			{
				SoundClassesToShow.Add(SoundClass);
			}
		}
	}

	for (int32 Index=0; Index < SoundClassesToShow.Num(); ++Index)
	{
		USoundClass* SoundClass = SoundClassesToShow[Index];
		if (Indent > 0)
		{
			Ar.Logf(TEXT("%s|- %s"), FCString::Spc(Indent*2), *SoundClass->GetName());
		}
		else
		{
			Ar.Logf(TEXT("%s"), *SoundClass->GetName());
		}
		for (int32 i = 0; i < SoundClass->ChildClasses.Num(); ++i)
		{
			if (SoundClass->ChildClasses[i])
			{
				ShowSoundClassHierarchy(Ar, SoundClass->ChildClasses[i], Indent+1);
			}
		}
	}
}

bool FAudioDevice::HandleDumpSoundInfoCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	using namespace AudioDeviceUtils;

	FAudioThreadSuspendContext AudioThreadSuspend;

	Ar.Logf(TEXT("Native Count: %d\nRealtime Count: %d\n"), PrecachedNative, PrecachedRealtime);
	float AverageSize = 0.0f;
	if (PrecachedNative != 0)
	{
		PrecachedNative = TotalNativeSize / PrecachedNative;
	}
	Ar.Logf(TEXT("Average Length: %.3g\nTotal Size: %d\nAverage Size: %.3g\n"), AverageNativeLength, TotalNativeSize, PrecachedNative);
	Ar.Logf(TEXT("Channel counts:\n"));
	for (auto CountIt = NativeChannelCount.CreateConstIterator(); CountIt; ++CountIt)
	{
		Ar.Logf(TEXT("\t%d: %d"),CountIt.Key(), CountIt.Value());
	}
	Ar.Logf(TEXT("Sample rate counts:\n"));
	for (auto SampleRateIt = NativeSampleRateCount.CreateConstIterator(); SampleRateIt; ++SampleRateIt)
	{
		Ar.Logf(TEXT("\t%d: %d"), SampleRateIt.Key(), SampleRateIt.Value());
	}
	return true;
}


bool FAudioDevice::HandleListSoundClassVolumesCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioThreadSuspendContext AudioThreadSuspend;

	Ar.Logf(TEXT("SoundClass Volumes: (Volume, Pitch)"));

	for (TMap<USoundClass*, FSoundClassProperties>::TIterator It(SoundClasses); It; ++It)
	{
		USoundClass* SoundClass = It.Key();
		if (SoundClass)
		{
			const FSoundClassProperties& CurClass = It.Value();

			Ar.Logf(TEXT("Cur (%3.2f, %3.2f) for SoundClass %s"), CurClass.Volume, CurClass.Pitch, *SoundClass->GetName());
		}
	}

	return true;
}

bool FAudioDevice::HandleListAudioComponentsCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioThreadSuspendContext AudioThreadSuspend;

	int32 Count = 0;
	Ar.Logf(TEXT("AudioComponent Dump"));
	for (TObjectIterator<UAudioComponent> It; It; ++It)
	{
		UAudioComponent* AudioComponent = *It;
		UObject* Outer = It->GetOuter();
		UObject* Owner = It->GetOwner();
		Ar.Logf(TEXT("    0x%p: %s, %s, %s, %s"),
			AudioComponent,
			*(It->GetPathName()),
			It->Sound ? *(It->Sound->GetPathName()) : TEXT("NO SOUND"),
			Outer ? *(Outer->GetPathName()) : TEXT("NO OUTER"),
			Owner ? *(Owner->GetPathName()) : TEXT("NO OWNER"));
		Ar.Logf(TEXT("        bAutoDestroy....................%s"), AudioComponent->bAutoDestroy ? TEXT("true") : TEXT("false"));
		Ar.Logf(TEXT("        bStopWhenOwnerDestroyed.........%s"), AudioComponent->bStopWhenOwnerDestroyed ? TEXT("true") : TEXT("false"));
		Ar.Logf(TEXT("        bShouldRemainActiveIfDropped....%s"), AudioComponent->bShouldRemainActiveIfDropped ? TEXT("true") : TEXT("false"));
		Ar.Logf(TEXT("        bIgnoreForFlushing..............%s"), AudioComponent->bIgnoreForFlushing ? TEXT("true") : TEXT("false"));
		Count++;
	}
	Ar.Logf(TEXT("AudioComponent Total = %d"), Count);

	Ar.Logf(TEXT("AudioDevice 0x%p has %d ActiveSounds"),
		this, ActiveSounds.Num());
	for (int32 ASIndex = 0; ASIndex < ActiveSounds.Num(); ASIndex++)
	{
		const FActiveSound* ActiveSound = ActiveSounds[ASIndex];
		UAudioComponent* AComp = UAudioComponent::GetAudioComponentFromID(ActiveSound->GetAudioComponentID());
		if (AComp)
		{
			Ar.Logf(TEXT("    0x%p: %4d - %s, %s, %s, %s"),
				AComp,
				ASIndex,
				*(AComp->GetPathName()),
				ActiveSound->Sound ? *(ActiveSound->Sound->GetPathName()) : TEXT("NO SOUND"),
				AComp->GetOuter() ? *(AComp->GetOuter()->GetPathName()) : TEXT("NO OUTER"),
				AComp->GetOwner() ? *(AComp->GetOwner()->GetPathName()) : TEXT("NO OWNER"));
		}
		else
		{
			Ar.Logf(TEXT("    %4d - %s, %s"),
				ASIndex,
				ActiveSound->Sound ? *(ActiveSound->Sound->GetPathName()) : TEXT("NO SOUND"),
				TEXT("NO COMPONENT"));
		}
	}
	return true;
}

bool FAudioDevice::HandleListSoundDurationsCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("Sound,Duration,Channels"));
	for (TObjectIterator<USoundWave> It; It; ++It)
	{
		USoundWave* SoundWave = *It;
		Ar.Logf(TEXT("%s,%f,%i"), *SoundWave->GetPathName(), SoundWave->Duration, SoundWave->NumChannels);
	}
	return true;
}

bool FAudioDevice::HandleSetBaseSoundMixCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Ar.Logf(TEXT("Setting base sound mix '%s'"), Cmd);
	const FName NewMix = FName(Cmd);
	USoundMix* SoundMix = nullptr;

	for (TObjectIterator<USoundMix> It; It; ++It)
	{
		if (NewMix == It->GetFName())
		{
			SoundMix = *It;
			break;
		}
	}

	if (SoundMix)
	{
		SetBaseSoundMix(SoundMix);
	}
	else
	{
		Ar.Logf(TEXT("Unknown SoundMix: %s"), *NewMix.ToString());
	}
	return true;
}

bool FAudioDevice::HandleIsolateDryAudioCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("Dry audio isolated"));
	SetMixDebugState(DEBUGSTATE_IsolateDryAudio);
	return true;
}

bool FAudioDevice::HandleIsolateReverbCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("Reverb audio isolated"));
	SetMixDebugState(DEBUGSTATE_IsolateReverb);
	return true;
}

bool FAudioDevice::HandleTestLPFCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("LPF set to max for all sources"));
	SetMixDebugState(DEBUGSTATE_TestLPF);
	return true;
}

bool FAudioDevice::HandleTestHPFCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("HPF set to max for all sources"));
	SetMixDebugState(DEBUGSTATE_TestHPF);
	return true;
}

bool FAudioDevice::HandleTestLFEBleedCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("LFEBleed set to max for all sources"));
	SetMixDebugState(DEBUGSTATE_TestLFEBleed);
	return true;
}

bool FAudioDevice::HandleDisableLPFCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("LPF disabled for all sources"));
	SetMixDebugState(DEBUGSTATE_DisableLPF);
	return true;
}

bool FAudioDevice::HandleDisableHPFCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("HPF disabled for all sources"));
	SetMixDebugState(DEBUGSTATE_DisableHPF);
	return true;
}

bool FAudioDevice::HandleDisableRadioCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	EnableRadioEffect(false);
	return true;
}

bool FAudioDevice::HandleEnableRadioCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	EnableRadioEffect(true);
	return true;
}

bool FAudioDevice::HandleResetSoundStateCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("All volumes reset to their defaults; all test filters removed"));
	SetMixDebugState(DEBUGSTATE_None);
	return true;
}

bool FAudioDevice::HandleToggleSpatializationExtensionCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	SetSpatializationInterfaceEnabled(!bSpatializationInterfaceEnabled);

	return true;
}

bool FAudioDevice::HandleEnableHRTFForAllCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	SetHRTFEnabledForAll(!bHRTFEnabledForAll_OnGameThread);

	return true;
}

bool FAudioDevice::HandleSoloCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Apply the solo to the given device
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	if (DeviceManager)
	{
		DeviceManager->SetSoloDevice(DeviceID);
	}
	return true;
}

bool FAudioDevice::HandleClearSoloCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	if (DeviceManager)
	{
		DeviceManager->SetSoloDevice(INDEX_NONE);
	}
	return true;
}

bool FAudioDevice::HandlePlayAllPIEAudioCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	if (DeviceManager)
	{
		DeviceManager->TogglePlayAllDeviceAudio();
	}
	return true;
}

bool FAudioDevice::HandleAudio3dVisualizeCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	if (DeviceManager)
	{
		DeviceManager->ToggleVisualize3dDebug();
	}
	return true;
}

void FAudioDevice::HandleAudioSoloCommon(const TCHAR* Cmd, FOutputDevice& Ar, FToggleSoloPtr FPtr)
{
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	if (DeviceManager)
	{
		bool bExclusive = true;
		if (FParse::Param(Cmd, TEXT("nonexclusive")))
		{
			bExclusive = false;
		}
		TArray<FString> Args;
		FString(Cmd).ParseIntoArray(Args, TEXT(" "));
		if (Args.Num())
		{
			(DeviceManager->GetDebugger().*FPtr)(*Args[0], bExclusive);
		}
		else if(bExclusive)
		{
			// If we are exclusive and no argument is passed, pass NAME_None to clear the current state.
			(DeviceManager->GetDebugger().*FPtr)(NAME_None, true);
		}
	}
}

bool FAudioDevice::HandleAudioSoloSoundClass(const TCHAR* Cmd, FOutputDevice& Ar)
{
	HandleAudioSoloCommon(Cmd, Ar, &Audio::FAudioDebugger::ToggleSoloSoundClass);
	return true;
}

bool FAudioDevice::HandleAudioSoloSoundWave(const TCHAR* Cmd, FOutputDevice& Ar)
{	
	HandleAudioSoloCommon(Cmd, Ar, &Audio::FAudioDebugger::ToggleSoloSoundWave);
	return true;
}

bool FAudioDevice::HandleAudioSoloSoundCue(const TCHAR* Cmd, FOutputDevice& Ar)
{
	HandleAudioSoloCommon(Cmd, Ar, &Audio::FAudioDebugger::ToggleSoloSoundCue);
	return true;
}

bool FAudioDevice::HandleAudioMixerDebugSound(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	if (DeviceManager)
	{
		DeviceManager->GetDebugger().SetAudioMixerDebugSound(Cmd);
	}
	return true;
}

bool FAudioDevice::HandleAudioDebugSound(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	if (DeviceManager)
	{
		DeviceManager->GetDebugger().SetAudioDebugSound(Cmd);
	}
	return true;
}

bool FAudioDevice::HandleSoundClassFixup(const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_EDITOR
	// Get asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> AssetDataArray;
	AssetRegistryModule.Get().GetAssetsByClass(USoundClass::StaticClass()->GetClassPathName(), AssetDataArray);

	static const FString EngineDir = "/Engine/";
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
	TArray<FAssetRenameData> RenameData;
	for (FAssetData AssetData : AssetDataArray)
	{
		USoundClass* SoundClass = Cast<USoundClass>(AssetData.GetAsset());
		if (SoundClass != nullptr && !SoundClass->GetPathName().Contains(EngineDir))
		{
			// If this sound class is within another sound class package, create a new uniquely named sound class
			FString OutermostFullName = SoundClass->GetOutermost()->GetName();
			FString ExistingSoundClassFullName = SoundClass->GetPathName();
			int32 CharPos = INDEX_NONE;

			FString OutermostShortName = FPaths::GetCleanFilename(OutermostFullName);

			OutermostShortName = FString::Printf(TEXT("%s.%s"), *OutermostShortName, *OutermostShortName);

			FString ExistingSoundClassShortName = FPaths::GetCleanFilename(ExistingSoundClassFullName);
			if (ExistingSoundClassShortName != OutermostShortName)
			{
				// Construct a proper new asset name/path
				FString ExistingSoundClassPath = ExistingSoundClassFullName.Left(CharPos);

				ExistingSoundClassShortName.FindLastChar('.', CharPos);

				// Get the name of the new sound class
				FString NewSoundClassName = ExistingSoundClassShortName.RightChop(CharPos + 1);

				const FString PackagePath = FPackageName::GetLongPackagePath(AssetData.GetAsset()->GetOutermost()->GetName());

				// Use the asset tool module to get a unique name based on the existing name
 				FString OutNewPackageName;
				FString OutAssetName;
 				AssetToolsModule.Get().CreateUniqueAssetName(PackagePath + "/" + NewSoundClassName, TEXT(""), OutNewPackageName, OutAssetName);

				const FString LongPackagePath = FPackageName::GetLongPackagePath(OutNewPackageName);

				// Immediately perform the rename since there could be a naming conflict in the list and CreateUniqueAssetName won't be able to resolve
				// unless the assets are renamed immediately
				RenameData.Reset();
				RenameData.Add(FAssetRenameData(AssetData.GetAsset(), LongPackagePath, OutAssetName));
				AssetToolsModule.Get().RenameAssetsWithDialog(RenameData);
			}
		}
	}
	return true;
#else
	return false;
#endif
}

bool FAudioDevice::HandleAudioMemoryInfo(const TCHAR* Cmd, FOutputDevice& Ar)
{
	struct FSoundWaveInfo
	{
		USoundWave* SoundWave;
		FResourceSizeEx ResourceSize;
		FString SoundGroupName;
		float Duration;

		enum class ELoadingType : uint8
		{
			CompressedInMemory,
			DecompressedInMemory,
			Streaming
		};

		// Whether this audio is decompressed in memory, decompressed in realtime, or streamed.
		ELoadingType LoadingType;

		// This is the maximum amount of the cache this asset could take up at any given time,
		// that could potentially not be removed if the sound is retained or currently playing.
		uint32 MaxUnevictableSizeInCache;

		// This is the total amount of compressed audio data that could be loaded in the cache.
		uint32 PotentialTotalSizeInCache;

		FSoundWaveInfo(USoundWave* InSoundWave, FResourceSizeEx InResourceSize, const FString& InSoundGroupName, float InDuration, ELoadingType InLoadingType, uint32 InMaxUnevictableSizeInCache, uint32 InPotentialTotalSizeInCache)
			: SoundWave(InSoundWave)
			, ResourceSize(InResourceSize)
			, SoundGroupName(InSoundGroupName)
			, Duration(InDuration)
			, LoadingType(InLoadingType)
			, MaxUnevictableSizeInCache(InMaxUnevictableSizeInCache)
			, PotentialTotalSizeInCache(InPotentialTotalSizeInCache)
		{}
	};

	using ELoadingType = FSoundWaveInfo::ELoadingType;

	struct FSoundWaveGroupInfo
	{
		FResourceSizeEx ResourceSize;
		FResourceSizeEx CompressedResourceSize;

		FSoundWaveGroupInfo()
			: ResourceSize()
			, CompressedResourceSize()
		{}
	};

	// Alpha sort the objects by path name
	struct FCompareSoundWave
	{
		FORCEINLINE bool operator()(const FSoundWaveInfo& A, const FSoundWaveInfo& B) const
		{
			return A.SoundWave->GetPathName() < B.SoundWave->GetPathName();
		}
	};

	// Default to writing our own .csv file unless -log is present
	const bool bLogOutputToFile = !FParse::Param(Cmd, TEXT("LOG"));

	FOutputDevice* ReportAr = &Ar;
	FArchive* FileAr = nullptr;
	FOutputDeviceArchiveWrapper* FileArWrapper = nullptr;

	if (bLogOutputToFile)
	{
		const FString PathName = *(FPaths::ProfilingDir() + TEXT("MemReports/"));
		IFileManager::Get().MakeDirectory(*PathName);

		const FString Filename = CreateProfileFilename(TEXT("_audio_memreport.csv"), true);
		const FString FilenameFull = PathName + Filename;

		FileAr = IFileManager::Get().CreateDebugFileWriter(*FilenameFull);
		FileArWrapper = new FOutputDeviceArchiveWrapper(FileAr);
		ReportAr = FileArWrapper;

		UE_LOGF(LogEngine, Log, "AudioMemReport: saving to %ls", *FilenameFull);
	}
	else
	{
		UE_LOGF(LogEngine, Log, "Use command \"AudioMemReport -log\" to output to client logs or the passed through FOutputDevice.");
	}

	// Get the sound wave class
	UClass* SoundWaveClass = nullptr;
	ParseObject<UClass>(TEXT("class=SoundWave"), TEXT("CLASS="), SoundWaveClass, nullptr);

	TArray<FSoundWaveInfo> SoundWaveObjects;
	TMap<FString, FSoundWaveGroupInfo> SoundWaveGroupSizes;
	TArray<FString> SoundWaveGroupFolders;

	// Grab the list of folders to specifically track memory usage for
	const FConfigSection* TrackedFolders = GConfig->GetSection(TEXT("AudioMemReportFolders"), 0, GEngineIni);
	if (TrackedFolders)
	{
		for (FConfigSectionMap::TConstIterator It(*TrackedFolders); It; ++It)
		{
			const FString& SoundFolder = *It.Value().GetValue();
			SoundWaveGroupSizes.Add(SoundFolder, FSoundWaveGroupInfo());
			SoundWaveGroupFolders.Add(SoundFolder);
		}
	}

	FResourceSizeEx TotalResourceSize;
	FResourceSizeEx CompressedResourceSize;
	FResourceSizeEx DecompressedResourceSize;
	int32 CompressedResourceCount = 0;

	if (SoundWaveClass != nullptr)
	{
		// Loop through all objects and find only sound wave objects
		for (TObjectIterator<USoundWave> It; It; ++It)
		{
			if (It->IsTemplate(RF_ClassDefaultObject))
			{
				continue;
			}

			// Get the resource size of the sound wave
			FResourceSizeEx TrueResourceSize = FResourceSizeEx(EResourceSizeMode::Exclusive);
			It->GetResourceSizeEx(TrueResourceSize);
			if (TrueResourceSize.GetTotalMemoryBytes() == 0)
			{
				continue;
			}

			USoundWave* SoundWave = *It;

			const FSoundGroup& SoundGroup = GetDefault<USoundGroups>()->GetSoundGroup(SoundWave->SoundGroup);

			// Determine whether this asset is streaming compressed data from disk, decompressed in realtime, or fully decompressed on load.
			ELoadingType LoadType = SoundWave->IsStreaming()? ELoadingType::Streaming : ELoadingType::CompressedInMemory;
			
			FString SoundGroupName;
			switch (SoundWave->SoundGroup)
			{
				case ESoundGroup::SOUNDGROUP_Default:
					SoundGroupName = TEXT("Default");
					break;

				case ESoundGroup::SOUNDGROUP_Effects:
					SoundGroupName = TEXT("Effects");
					break;

				case ESoundGroup::SOUNDGROUP_UI:
					SoundGroupName = TEXT("UI");
					break;

				case ESoundGroup::SOUNDGROUP_Music:
					SoundGroupName = TEXT("Music");
					break;

				case ESoundGroup::SOUNDGROUP_Voice:
					SoundGroupName = TEXT("Voice");
					break;

				default:
					SoundGroupName = SoundGroup.DisplayName;
					break;
			}

			// Calculate streaming cache size usage. 
			uint32 MaxSizeInCache = 0;
			uint32 MaxUnevictableSize = 0;
			SoundWave->GetChunkSizeStats(MaxSizeInCache, MaxUnevictableSize);

			// Add the info to the SoundWaveObjects array
			SoundWaveObjects.Add(FSoundWaveInfo(SoundWave, TrueResourceSize, SoundGroupName, SoundWave->Duration, LoadType, MaxUnevictableSize, MaxSizeInCache));

			// Track total resource usage
			TotalResourceSize += TrueResourceSize;

			if (LoadType == ELoadingType::DecompressedInMemory)
			{
				DecompressedResourceSize += TrueResourceSize;
				++CompressedResourceCount;
			}
			else if (LoadType == ELoadingType::CompressedInMemory)
			{
				CompressedResourceSize += TrueResourceSize;
			}

			// Get the sound object path
			FString SoundWavePath = SoundWave->GetPathName();

			// Now track the resource size according to all the sub-directories
			FString SubDir;
			int32 Index = 0;

			for (int32 i = 0; i < SoundWavePath.Len(); ++i)
			{
				if (SoundWavePath[i] == '/')
				{
					if (SubDir.Len() > 0)
					{
						FSoundWaveGroupInfo* SubDirSize = SoundWaveGroupSizes.Find(SubDir);
						if (SubDirSize)
						{
							SubDirSize->ResourceSize += TrueResourceSize;
							if (LoadType == ELoadingType::CompressedInMemory)
							{
								SubDirSize->CompressedResourceSize += TrueResourceSize;
							}
						}
					}
					SubDir = TEXT("");
				}
				else
				{
					SubDir.AppendChar(SoundWavePath[i]);
				}
			}
		}

		ReportAr->Log(TEXT("Sound Wave Memory Report"));
		ReportAr->Log(TEXT(""));

		FString StreamingMemoryReport = IStreamingManager::Get().GetAudioStreamingManager().GenerateMemoryReport();

		ReportAr->Log(TEXT("\n/*******************/\n"));
		ReportAr->Log(TEXT("Streaming Audio Info:"));
		ReportAr->Log(*StreamingMemoryReport);
		ReportAr->Log(TEXT("\n/*******************/\n"));

		if (SoundWaveObjects.Num())
		{
			// Alpha sort the sound wave objects
			SoundWaveObjects.Sort(FCompareSoundWave());

			// Log the sound wave objects

			ReportAr->Logf(TEXT("Memory (MB),Count"));
			ReportAr->Logf(TEXT("Total,%.3f,%d"), TotalResourceSize.GetTotalMemoryBytes() / 1024.f / 1024.f, SoundWaveObjects.Num());
			ReportAr->Logf(TEXT("Decompressed,%.3f,%d"), DecompressedResourceSize.GetTotalMemoryBytes() / 1024.f / 1024.f, CompressedResourceCount);
			ReportAr->Logf(TEXT("Compressed,%.3f,%d"), CompressedResourceSize.GetTotalMemoryBytes() / 1024.f / 1024.f, SoundWaveObjects.Num() - CompressedResourceCount);

			if (SoundWaveGroupFolders.Num())
			{
				ReportAr->Log(TEXT(""));
				ReportAr->Log(TEXT("Memory Usage and Count for Specified Folders (Folders defined in [AudioMemReportFolders] section in DefaultEngine.ini file):"));
				ReportAr->Log(TEXT(""));
				ReportAr->Logf(TEXT("%s,%s,%s"), TEXT("Directory"), TEXT("Total (MB)"), TEXT("Compressed (MB)"));
				for (const FString& SoundWaveGroupFolder : SoundWaveGroupFolders)
				{
					FSoundWaveGroupInfo* SubDirSize = SoundWaveGroupSizes.Find(SoundWaveGroupFolder);
					check(SubDirSize);
					ReportAr->Logf(TEXT("%s,%.2f,%.2f"), *SoundWaveGroupFolder, SubDirSize->ResourceSize.GetTotalMemoryBytes() / 1024.0f / 1024.0f, SubDirSize->CompressedResourceSize.GetTotalMemoryBytes() / 1024.0f / 1024.0f);
				}
			}

			ReportAr->Log(TEXT(""));
			ReportAr->Log(TEXT("All Sound Wave Objects Sorted Alphebetically:"));
			ReportAr->Log(TEXT(""));

			ReportAr->Logf(TEXT("%s,%s,%s,%s,%s,%s,%s,%s"), TEXT("SoundWave"), TEXT("KB"), TEXT("MB"), TEXT("SoundGroup"), TEXT("Duration"), TEXT("CompressionState"), TEXT("Max Size in Cache (Unevictable KB)"), TEXT("Max Size In Cache (Total KB)"));
			for (const FSoundWaveInfo& Info : SoundWaveObjects)
			{
				float Kbytes = Info.ResourceSize.GetTotalMemoryBytes() / 1024.0f;

				FString LoadingTypeString;

				switch (Info.LoadingType)
				{
				case ELoadingType::CompressedInMemory:
					LoadingTypeString = TEXT("Compressed");
					break;
				case ELoadingType::DecompressedInMemory:
					LoadingTypeString = TEXT("Decompressed");
					break;
				case ELoadingType::Streaming:
					LoadingTypeString = TEXT("Streaming");
				default:
					break;
				}

				ReportAr->Logf(TEXT("%s,%.2f,%.2f,%s,%.2f,%s,%.2f,%.2f"), *Info.SoundWave->GetPathName(), Kbytes, Kbytes / 1024.0f, *Info.SoundGroupName, Info.Duration, *LoadingTypeString, Info.MaxUnevictableSizeInCache / 1024.0f, Info.PotentialTotalSizeInCache / 1024.0f);
			}
		}

	}

	if (FileArWrapper != nullptr)
	{
		// Shutdown and free archive resources
		FileArWrapper->TearDown();
	}
	delete FileArWrapper;
	delete FileAr;

	return true;
}

bool FAudioDevice::HandleResetAllDynamicSoundVolumesCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
	{
		DeviceManager->ResetAllDynamicSoundVolumes();
	}
	return true;
}

bool FAudioDevice::HandleResetDynamicSoundVolumeCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
	{
		FName SoundName;
		if (!FParse::Value(Cmd, TEXT("Name="), SoundName))
		{
			return false;
		}

		// Optional: Defaults to Cue
		FString SoundTypeStr;
		ESoundType SoundType = ESoundType::Cue;
		if (FParse::Value(Cmd, TEXT("Type="), SoundTypeStr))
		{
			if (SoundTypeStr == TEXT("Wave"))
			{
				SoundType = ESoundType::Wave;
			}
			else if (SoundTypeStr == TEXT("Class"))
			{
				SoundType = ESoundType::Class;
			}
		}

		DeviceManager->ResetDynamicSoundVolume(SoundType, SoundName);
	}
	return true;
}

bool FAudioDevice::HandleGetDynamicSoundVolumeCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!GEngine)
	{
		return false;
	}

	if (const FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
	{
		FName SoundName;
		if (!FParse::Value(Cmd, TEXT("Name="), SoundName))
		{
			return false;
		}

		// Optional: Defaults to Cue
		FString SoundTypeStr;
		ESoundType SoundType = ESoundType::Cue;
		if (FParse::Value(Cmd, TEXT("Type="), SoundTypeStr))
		{
			if (SoundTypeStr == TEXT("Wave"))
			{
				SoundType = ESoundType::Wave;
			}
			else if (SoundTypeStr == TEXT("Class"))
			{
				SoundType = ESoundType::Class;
			}
		}

		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.GetDynamicSoundVolume"), STAT_AudioGetDynamicSoundVolume, STATGROUP_AudioThreadCommands);

			const ESoundType InSoundType = SoundType;
			const FName InSoundName = SoundName;
			FAudioThread::RunCommandOnAudioThread([InSoundType, InSoundName]()
			{
				if (!GEngine)
				{
					return;
				}
				if (const FAudioDeviceManager* InDeviceManager = GEngine->GetAudioDeviceManager())
				{
					const float Volume = InDeviceManager->GetDynamicSoundVolume(InSoundType, InSoundName);
					UE_LOGF(LogAudio, Display, "'%ls' Dynamic Volume: %.4f", *InSoundName.GetPlainNameString(), Volume);
				}
			}, GET_STATID(STAT_AudioGetDynamicSoundVolume));
		}
		else
		{
		const float Volume = DeviceManager->GetDynamicSoundVolume(SoundType, SoundName);
		FString Msg = FString::Printf(TEXT("'%s' Dynamic Volume: %.4f"), *SoundName.GetPlainNameString(), Volume);
		Ar.Logf(TEXT("%s"), *Msg);
	}
	}
	return true;
}

bool FAudioDevice::HandleSetDynamicSoundCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
	{
		FName SoundName;
		if (!FParse::Value(Cmd, TEXT("Name="), SoundName))
		{
			return false;
		}

		// Optional: Defaults to Cue
		FString SoundTypeStr;
		ESoundType SoundType = ESoundType::Cue;
		if (FParse::Value(Cmd, TEXT("Type="), SoundTypeStr))
		{
			if (SoundTypeStr == TEXT("Wave"))
			{
				SoundType = ESoundType::Wave;
			}
			else if (SoundTypeStr == TEXT("Class"))
			{
				SoundType = ESoundType::Class;
			}
		}

		float Volume;
		if (!FParse::Value(Cmd, TEXT("Vol="), Volume))
		{
			return false;
		}

		DeviceManager->SetDynamicSoundVolume(SoundType, SoundName, Volume);
	}
	return true;
}
#endif // !UE_BUILD_SHIPPING

#if UE_AUDIO_PROFILERTRACE_ENABLED
void FAudioDevice::OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
{
	for (const FActiveSound* ActiveSound : ActiveSounds)
	{
		if (ActiveSound == nullptr)
		{
			continue;
		}

		SendSoundIsAlivePingTraceMessage(*ActiveSound);
	}

	for (auto& [ActiveSound, VirtualLoop] : VirtualLoops)
	{
		VirtualLoop.OnTraceStarted();
	}
}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

bool FAudioDevice::IsHRTFEnabledForAll() const
{
	if (IsInAudioThread())
	{
		return (bHRTFEnabledForAll || EnableBinauralAudioForAllSpatialSoundsCVar == 1) && IsSpatializationPluginEnabled();
	}

	check(IsInGameThread());
	return (bHRTFEnabledForAll_OnGameThread || EnableBinauralAudioForAllSpatialSoundsCVar == 1) && IsSpatializationPluginEnabled();
}

bool FAudioDevice::IsHRTFDisabled() const
{
	if (IsInAudioThread())
	{
		return (bHRTFDisabled || DisableBinauralSpatializationCVar == 1);
	}

	check(IsInGameThread());
	return (bHRTFDisabled_OnGameThread || DisableBinauralSpatializationCVar == 1);
}

void FAudioDevice::SetMixDebugState(EDebugState InDebugState)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetMixDebugState"), STAT_AudioSetMixDebugState, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, InDebugState]()
		{
			AudioDevice->SetMixDebugState(InDebugState);

		}, GET_STATID(STAT_AudioSetMixDebugState));

		return;
	}

	DebugState = InDebugState;
}

#if UE_ALLOW_EXEC_COMMANDS
bool FAudioDevice::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !UE_BUILD_SHIPPING
	auto ParseAudioExecCmd = [](const TCHAR** InCmd, const TCHAR* InMatch)
	{
		if (FParse::Command(InCmd, InMatch))
		{
			UE_LOGF(LogAudio, Warning, "The Exec command '%ls' is deprecated. Use 'au.Debug.%ls' instead", InMatch, InMatch);
			return true;
		}

		const FString InMatchFull = FString::Printf(TEXT("au.Debug.%s"), InMatch);
		return FParse::Command(InCmd, *InMatchFull);
	};

	if (ParseAudioExecCmd(&Cmd, TEXT("DumpSoundInfo")))
	{
		return HandleDumpSoundInfoCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ListWaves")))
	{
		return HandleListWavesCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ListSoundClasses")))
	{
		return HandleListSoundClassesCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ShowSoundClassHierarchy")))
	{
		return HandleShowSoundClassHierarchyCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ListSoundClassVolumes")))
	{
		return HandleListSoundClassVolumesCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ListAudioComponents")))
	{
		return HandleListAudioComponentsCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ListSoundDurations")))
	{
		return HandleListSoundDurationsCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("SetBaseSoundMix")))
	{
		return HandleSetBaseSoundMixCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("IsolateDryAudio")))
	{
		return HandleIsolateDryAudioCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("IsolateReverb")))
	{
		return HandleIsolateReverbCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("TestLPF")))
	{
		return HandleTestLPFCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("TestLFEBleed")))
	{
		return HandleTestLPFCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("DisableLPF")))
	{
		return HandleDisableLPFCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("DisableHPF")))
	{
		return HandleDisableHPFCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("DisableRadio")))
	{
		return HandleDisableRadioCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("EnableRadio")))
	{
		return HandleEnableRadioCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ResetSoundState")))
	{
		return HandleResetSoundStateCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ToggleSpatExt")))
	{
		return HandleToggleSpatializationExtensionCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ToggleHRTFForAll")))
	{
		return HandleEnableHRTFForAllCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("SoloAudio")))
	{
		return HandleSoloCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("ClearSoloAudio")))
	{
		return HandleClearSoloCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("PlayAllPIEAudio")))
	{
		return HandlePlayAllPIEAudioCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("Audio3dVisualize")))
	{
		return HandleAudio3dVisualizeCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioSoloSoundClass")))
	{
		return HandleAudioSoloSoundClass(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioSoloSoundWave")))
	{
		return HandleAudioSoloSoundWave(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioSoloSoundCue")))
	{
		return HandleAudioSoloSoundCue(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioMemReport")))
	{
		return HandleAudioMemoryInfo(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioMixerDebugSound")))
	{
		return HandleAudioMixerDebugSound(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioDebugSound")))
	{
		return HandleAudioDebugSound(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("SoundClassFixup")))
	{
		return HandleSoundClassFixup(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioResetDynamicSoundVolume")))
	{
		return HandleResetDynamicSoundVolumeCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioResetAllDynamicSoundVolumes")))
	{
		return HandleResetAllDynamicSoundVolumesCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioGetDynamicSoundVolume")))
	{
		return HandleGetDynamicSoundVolumeCommand(Cmd, Ar);
	}
	else if (ParseAudioExecCmd(&Cmd, TEXT("AudioSetDynamicSoundVolume")))
	{
		return HandleSetDynamicSoundCommand(Cmd, Ar);
	}
#endif // !UE_BUILD_SHIPPING

	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

void FAudioDevice::InitSoundClasses()
{
	// Reset the maps of sound class properties
	for (TObjectIterator<USoundClass> It; It; ++It)
	{
		USoundClass* SoundClass = *It;
		SoundClasses.Add(SoundClass, SoundClass->Properties);

		// Set the dynamic properties
		FSoundClassDynamicProperties DynamicProperty;
		DynamicProperty.AttenuationScaleParam.Set(SoundClass->Properties.AttenuationDistanceScale, 0.0f);

		DynamicSoundClassProperties.Add(SoundClass, DynamicProperty);
	}

	// Propagate the properties down the hierarchy
	ParseSoundClasses(0.0f);
}

void FAudioDevice::InitSoundSources()
{
	if (Sources.Num() == 0)
	{
		// now create platform specific sources
		const int32 SourceMax = GetMaxSources();
		for (int32 SourceIndex = 0; SourceIndex < SourceMax; SourceIndex++)
		{
			FSoundSource* Source = CreateSoundSource();
			Source->InitializeSourceEffects(SourceIndex);

			Sources.Add(Source);
			FreeSources.Add(Source);
		}
	}
}

void FAudioDevice::InitializeSubsystemCollection()
{
	UE_LOGF(LogAudio, Log, "Initializing audio subsystem collection for audio device with id %d", DeviceID);

	SubsystemCollectionRoot.Reset(NewObject<UAudioSubsystemCollectionRoot>(GetTransientPackage()));
	check(SubsystemCollectionRoot.IsValid());

	SubsystemCollectionRoot->SetAudioDeviceID(DeviceID);
	SubsystemCollection.Initialize(SubsystemCollectionRoot.Get());
}

void FAudioDevice::SetDefaultBaseSoundMix(USoundMix* SoundMix)
{
	if (IsInGameThread() && SoundMix == nullptr)
	{
		const FSoftObjectPath DefaultBaseSoundMixName = GetDefault<UAudioSettings>()->DefaultBaseSoundMix;
		if (DefaultBaseSoundMixName.IsValid())
		{
			SoundMix = LoadObject<USoundMix>(nullptr, *DefaultBaseSoundMixName.ToString());
		}
	}

	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetDefaultBaseSoundMix"), STAT_AudioSetDefaultBaseSoundMix, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, SoundMix]()
		{
			AudioDevice->SetDefaultBaseSoundMix(SoundMix);

		}, GET_STATID(STAT_AudioSetDefaultBaseSoundMix));

		return;
	}

	DefaultBaseSoundMix = SoundMix;
	SetBaseSoundMix(SoundMix);
}

void FAudioDevice::RemoveSoundMix(USoundMix* SoundMix)
{
	check(IsInAudioThread());

	if (SoundMix)
	{
		// Not sure if we will ever destroy the default base SoundMix
		if (SoundMix == DefaultBaseSoundMix)
		{
			DefaultBaseSoundMix = nullptr;
		}

		ClearSoundMix(SoundMix);

		// Try setting to global default if base SoundMix has been cleared
		if (BaseSoundMix == nullptr)
		{
			SetBaseSoundMix(DefaultBaseSoundMix);
		}
	}
}

void FAudioDevice::RecurseIntoSoundClasses(USoundClass* CurrentClass, FSoundClassProperties& ParentProperties)
{
	// Iterate over all child nodes and recurse.
	for (USoundClass* ChildClass : CurrentClass->ChildClasses)
	{
		// Look up class and propagated properties.
		FSoundClassProperties* Properties = SoundClasses.Find(ChildClass);

		// Should never be NULL for a properly set up tree.
		if (ChildClass)
		{
			if (Properties)
			{
				Properties->Volume *= ParentProperties.Volume;
				Properties->Pitch *= ParentProperties.Pitch;
				Properties->bIsUISound |= ParentProperties.bIsUISound;
				Properties->bIsMusic |= ParentProperties.bIsMusic;

				// Not all values propagate equally...
				// VoiceCenterChannelVolume, RadioFilterVolume, RadioFilterVolumeThreshold, bApplyEffects, BleedStereo, bReverb, and bCenterChannelOnly do not propagate (sub-classes can be non-zero even if parent class is zero)

				// ... and recurse into child nodes.
				RecurseIntoSoundClasses(ChildClass, *Properties);
			}
			else
			{
				UE_LOGF(LogAudio, Warning, "Couldn't find child class properties - sound class functionality will not work correctly! CurrentClass: %ls ChildClass: %ls", *CurrentClass->GetFullName(), *ChildClass->GetFullName());
			}
		}
	}
}

void FAudioDevice::UpdateHighestPriorityReverb()
{
	check(IsInGameThread());

	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.UpdateHighestPriorityReverb"), STAT_AudioUpdateHighestPriorityReverb, STATGROUP_AudioThreadCommands);

	FAudioDevice* AudioDevice = this;

	if (ActivatedReverbs.Num() > 0)
	{
		ActivatedReverbs.ValueSort([](const FActivatedReverb& A, const FActivatedReverb& B) { return A.Priority > B.Priority; });

		// Use shared ptr to ensure that pending reverb is protected from GC before it gets applied on the audio thread.
		PendingActivatedReverb = MakeShared<FActivatedReverb>(ActivatedReverbs.CreateConstIterator().Value());
		FAudioThread::RunCommandOnAudioThread([AudioDevice, NewActivatedReverbWeakPtr = TWeakPtr<FActivatedReverb>(PendingActivatedReverb)]()
		{
			if (TSharedPtr<FActivatedReverb> NewActivatedReverb = NewActivatedReverbWeakPtr.Pin())
			{
				FScopeLock AudioThreadReferenceLock(&AudioDevice->AudioThreadReferenceCritSec);
				AudioDevice->bHasActivatedReverb = true;
				AudioDevice->HighestPriorityActivatedReverb_AudioThread = *NewActivatedReverb;
			}
		}, GET_STATID(STAT_AudioUpdateHighestPriorityReverb));
	}
	else
	{
		PendingActivatedReverb.Reset();
		FAudioThread::RunCommandOnAudioThread([AudioDevice]()
		{
			FScopeLock AudioThreadReferenceLock(&AudioDevice->AudioThreadReferenceCritSec);
			AudioDevice->bHasActivatedReverb = false;
			AudioDevice->HighestPriorityActivatedReverb_AudioThread = FActivatedReverb();
		}, GET_STATID(STAT_AudioUpdateHighestPriorityReverb));
	}
}

void FAudioDevice::ParseSoundClasses(float InDeltaTime)
{
	TArray<USoundClass*> RootSoundClasses;

	// Reset to known state - preadjusted by set class volume calls
	for (TMap<USoundClass*, FSoundClassProperties>::TIterator It(SoundClasses); It; ++It)
	{
		USoundClass* SoundClass = It.Key();
		if (SoundClass)
		{
			if (FSoundClassDynamicProperties* DynamicProperties = DynamicSoundClassProperties.Find(SoundClass))
			{
				DynamicProperties->AttenuationScaleParam.Update(InDeltaTime);
			}

			// Reset the property values
			It.Value() = SoundClass->Properties;
			if (SoundClass->ParentClass == NULL)
			{
				RootSoundClasses.Add(SoundClass);
			}
		}
	}

	for (int32 RootIndex = 0; RootIndex < RootSoundClasses.Num(); ++RootIndex)
	{
		USoundClass* RootSoundClass = RootSoundClasses[RootIndex];

		FSoundClassProperties* RootSoundClassProperties = SoundClasses.Find(RootSoundClass);
		if (RootSoundClass && RootSoundClassProperties)
		{
			// Follow the tree.
			RecurseIntoSoundClasses(RootSoundClass, *RootSoundClassProperties);
		}
	}
}

void FAudioDevice::RecursiveApplyAdjuster(const FSoundClassAdjuster& InAdjuster, USoundClass* InSoundClass)
{
	// Find the sound class properties so we can apply the adjuster
	// and find the sound class so we can recurse through the children
	FSoundClassProperties* Properties = SoundClasses.Find(InSoundClass);
	if (InSoundClass && Properties)
	{
		// Adjust this class
		Properties->Volume *= InAdjuster.VolumeAdjuster;
		Properties->Pitch *= InAdjuster.PitchAdjuster;
		Properties->VoiceCenterChannelVolume *= InAdjuster.VoiceCenterChannelVolumeAdjuster;

		// Only set the LPF frequency if the input adjuster is *less* than the sound class' property
		if (InAdjuster.LowPassFilterFrequency < Properties->LowPassFilterFrequency)
		{
			Properties->LowPassFilterFrequency = InAdjuster.LowPassFilterFrequency;
		}

		// Recurse through this classes children
		for (int32 ChildIdx = 0; ChildIdx < InSoundClass->ChildClasses.Num(); ++ChildIdx)
		{
			if (InSoundClass->ChildClasses[ ChildIdx ])
			{
				RecursiveApplyAdjuster(InAdjuster, InSoundClass->ChildClasses[ ChildIdx ]);
			}
		}
	}
	else
	{
		UE_LOGF(LogAudio, Display, "RecursiveApplyAdjuster failed, likely because we are clearing the level.");
	}
}

void FAudioDevice::UpdateConcurrency(TArray<FWaveInstance*>& WaveInstances, TArray<FActiveSound *>& ActiveSoundsCopy)
{
	// Now stop any sounds that are active that are in concurrency resolution groups that resolve by stopping quietest
	{
		SCOPE_CYCLE_COUNTER(STAT_AudioEvaluateConcurrency);
		ConcurrencyManager.UpdateSoundsToCull();
		ConcurrencyManager.UpdateVolumeScaleGenerations();
	}

	for (int32 i = ActiveSoundsCopy.Num() - 1; i >= 0; --i)
	{
		if (FActiveSound* ActiveSound = ActiveSoundsCopy[i])
		{
			if (!ActiveSound->bShouldStopDueToMaxConcurrency)
			{
				continue;
			}

			if (ActiveSound->FadeOut == FActiveSound::EFadeOut::Concurrency)
			{
				continue;
			}

			if (IsPendingStop(ActiveSound))
			{
				continue;
			}

			ConcurrencyManager.StopDueToVoiceStealing(*ActiveSound);
		}
	}

	// Remove all wave instances from the wave instance list that are stopping due to max concurrency.
	// Must be after checking if sound must fade out due to concurrency to avoid pre-maturally removing
	// wave instances prior to concurrency system marking as fading out.
	for (int32 i = WaveInstances.Num() - 1; i >= 0; --i)
	{
		if (WaveInstances[i]->ShouldStopDueToMaxConcurrency())
		{
			WaveInstances.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}

	// Must be completed after removing wave instances as it avoids an issue
	// where quiet loops can wrongfully scale concurrency ducking improperly if they continue
	// to attempt to be evaluated while being periodically realized to check volumes from virtualized.
	for (int32 i = 0; i < ActiveSoundsCopy.Num(); ++i)
	{
		if (FActiveSound* ActiveSound = ActiveSoundsCopy[i])
		{
			ActiveSound->UpdateConcurrencyVolumeScalars(GetGameDeltaTime());
		}
	}
}

bool FAudioDevice::ApplySoundMix(USoundMix* NewMix, FSoundMixState* SoundMixState)
{
	if (NewMix && SoundMixState)
	{
		UE_LOGF(LogAudio, Log, "FAudioDevice::ApplySoundMix(): %ls", *NewMix->GetName());

		SoundMixState->StartTime = GetAudioClock();
		SoundMixState->FadeInStartTime = SoundMixState->StartTime + NewMix->InitialDelay;
		SoundMixState->FadeInEndTime = SoundMixState->FadeInStartTime + NewMix->FadeInTime;
		SoundMixState->FadeOutStartTime = -1.0;
		SoundMixState->EndTime = -1.0;
		if (NewMix->Duration >= 0.0f)
		{
			SoundMixState->FadeOutStartTime = SoundMixState->FadeInEndTime + NewMix->Duration;
			SoundMixState->EndTime = SoundMixState->FadeOutStartTime + NewMix->FadeOutTime;
		}
		SoundMixState->InterpValue = 0.0f;

		// On sound mix application, there is no delta time
		const float InitDeltaTime = 0.0f;

		ApplyClassAdjusters(NewMix, SoundMixState->InterpValue, InitDeltaTime);

		return(true);
	}

	return(false);
}

void FAudioDevice::UpdateSoundMix(USoundMix* SoundMix, FSoundMixState* SoundMixState)
{
	// If this SoundMix will automatically end, add some more time
	if (SoundMixState->FadeOutStartTime >= 0.f)
	{
		SoundMixState->StartTime = GetAudioClock();

		// Don't need to reset the fade-in times since we don't want to retrigger fade-ins
		// But we need to update the fade out start and end times
		if (SoundMixState->CurrentState != ESoundMixState::Inactive)
		{
			SoundMixState->FadeOutStartTime = -1.0;
			SoundMixState->EndTime = -1.0;

			if (SoundMix->Duration >= 0.0f)
			{
				if (SoundMixState->CurrentState == ESoundMixState::FadingIn || SoundMixState->CurrentState == ESoundMixState::Active)
				{
					SoundMixState->FadeOutStartTime = SoundMixState->StartTime + SoundMix->FadeInTime + SoundMix->Duration;
					SoundMixState->EndTime = SoundMixState->FadeOutStartTime + SoundMix->FadeOutTime;

				}
				else if (SoundMixState->CurrentState == ESoundMixState::FadingOut || SoundMixState->CurrentState == ESoundMixState::AwaitingRemoval)
				{
					// Flip the state to fade in
					SoundMixState->CurrentState = ESoundMixState::FadingIn;

					SoundMixState->InterpValue = 0.0f;

					SoundMixState->FadeInStartTime = GetAudioClock() - SoundMixState->InterpValue * SoundMix->FadeInTime;
					SoundMixState->StartTime = SoundMixState->FadeInStartTime;

					SoundMixState->FadeOutStartTime = GetAudioClock() + SoundMix->FadeInTime + SoundMix->Duration;
					SoundMixState->EndTime = SoundMixState->FadeOutStartTime + SoundMix->FadeOutTime;
				}
			}
			else if (SoundMixState->CurrentState == ESoundMixState::FadingOut || SoundMixState->CurrentState == ESoundMixState::AwaitingRemoval)
			{
				// Pretend our fade time is starting now
				SoundMixState->StartTime = GetAudioClock();

				// Invert the current fade-out interp value to get a fade-in interp value
				float TargetInterpValue = 1.0f - SoundMixState->InterpValue;

				// Compute a fade in start time that would result in this target interp value so we avoid jumps
				SoundMixState->FadeInStartTime = SoundMixState->StartTime - TargetInterpValue * SoundMix->FadeInTime;

				SoundMixState->CurrentState = ESoundMixState::FadingIn;
				SoundMixState->FadeInEndTime = SoundMixState->FadeInStartTime + SoundMix->FadeInTime;
			}
		}
	}
}

void FAudioDevice::UpdatePassiveSoundMixModifiers(TArray<FWaveInstance*>& WaveInstances, int32 FirstActiveIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_UpdatePassiveSoundMixModifiers);

	TArray<USoundMix*> CurrPassiveSoundMixModifiers;

	// Find all passive SoundMixes from currently active wave instances
	for (int32 WaveIndex = FirstActiveIndex; WaveIndex < WaveInstances.Num(); WaveIndex++)
	{
		FWaveInstance* WaveInstance = WaveInstances[WaveIndex];
		if (WaveInstance)
		{
			USoundClass* SoundClass = WaveInstance->SoundClass;
			if (SoundClass)
			{
				const float WaveInstanceActualVolume = WaveInstance->GetVolumeWithDistanceAndOcclusionAttenuation() * WaveInstance->GetDynamicVolume();
				// Check each SoundMix individually for volume levels
				for (const FPassiveSoundMixModifier& PassiveSoundMixModifier : SoundClass->PassiveSoundMixModifiers)
				{
					if (WaveInstanceActualVolume >= PassiveSoundMixModifier.MinVolumeThreshold && WaveInstanceActualVolume <= PassiveSoundMixModifier.MaxVolumeThreshold)
					{
						// If the active sound is brand new, add to the new list...
 						if (WaveInstance->ActiveSound->PlaybackTime == 0.0f && PassiveSoundMixModifier.SoundMix)
 						{
							PushSoundMixModifier(PassiveSoundMixModifier.SoundMix, true, true);
 						}

						// Only add a unique sound mix modifier
						CurrPassiveSoundMixModifiers.AddUnique(PassiveSoundMixModifier.SoundMix);
					}
				}
			}
		}
	}

	// Push SoundMixes that weren't previously active
	for (USoundMix* CurrPassiveSoundMixModifier : CurrPassiveSoundMixModifiers)
	{
		if (PrevPassiveSoundMixModifiers.Find(CurrPassiveSoundMixModifier) == INDEX_NONE)
		{
			PushSoundMixModifier(CurrPassiveSoundMixModifier, true);
		}
	}

	// Pop SoundMixes that are no longer active
	for (int32 MixIdx = PrevPassiveSoundMixModifiers.Num() - 1; MixIdx >= 0; MixIdx--)
	{
		USoundMix* PrevPassiveSoundMixModifier = PrevPassiveSoundMixModifiers[MixIdx];
		if (CurrPassiveSoundMixModifiers.Find(PrevPassiveSoundMixModifier) == INDEX_NONE)
		{
			PopSoundMixModifier(PrevPassiveSoundMixModifier, true);
		}
	}

	PrevPassiveSoundMixModifiers = ObjectPtrWrap(CurrPassiveSoundMixModifiers);
}

bool FAudioDevice::TryClearingSoundMix(USoundMix* SoundMix, FSoundMixState* SoundMixState)
{
	if (SoundMix && SoundMixState)
	{
		// Only manually clear the sound mix if it's no longer referenced and if the duration was not set.
		// If the duration was set by sound designer, let the sound mix clear itself up automatically.
		if (SoundMix->Duration < 0.0f && SoundMixState->ActiveRefCount == 0 && SoundMixState->PassiveRefCount == 0 && SoundMixState->IsBaseSoundMix == false)
		{
			// do whatever is needed to remove influence of this SoundMix
			if (SoundMix->FadeOutTime > 0.f)
			{
				if (SoundMixState->CurrentState == ESoundMixState::Inactive)
				{
					// Haven't even started fading up, can kill immediately
					ClearSoundMix(SoundMix);
				}
				else if (SoundMixState->CurrentState == ESoundMixState::FadingIn)
				{
					// Currently fading up, force fade in to complete and start fade out from current fade level
					SoundMixState->FadeOutStartTime = GetAudioClock() - ((1.0f - SoundMixState->InterpValue) * SoundMix->FadeOutTime);
					SoundMixState->EndTime = SoundMixState->FadeOutStartTime + SoundMix->FadeOutTime;
					SoundMixState->StartTime = SoundMixState->FadeInStartTime = SoundMixState->FadeInEndTime = SoundMixState->FadeOutStartTime - 1.0;

					TryClearingEQSoundMix(SoundMix);
				}
				else if (SoundMixState->CurrentState == ESoundMixState::Active)
				{
					// SoundMix active, start fade out early
					SoundMixState->FadeOutStartTime = GetAudioClock();
					SoundMixState->EndTime = SoundMixState->FadeOutStartTime + SoundMix->FadeOutTime;

					TryClearingEQSoundMix(SoundMix);
				}
				else
				{
					// Already fading out, do nothing
				}
			}
			else
			{
				ClearSoundMix(SoundMix);
			}
			return true;
		}
	}

	return false;
}

bool FAudioDevice::TryClearingEQSoundMix(USoundMix* SoundMix)
{
	if (SoundMix && Effects && Effects->GetCurrentEQMix() == SoundMix)
	{
		USoundMix* NextEQMix = FindNextHighestEQPrioritySoundMix(SoundMix);
		if (NextEQMix)
		{
			// Need to ignore priority when setting as it will be less than current
			Effects->SetMixSettings(NextEQMix, true);
		}
		else
		{
			Effects->ClearMixSettings();
		}

		return true;
	}

	return false;
}

USoundMix* FAudioDevice::FindNextHighestEQPrioritySoundMix(USoundMix* IgnoredSoundMix)
{
	// find the mix with the next highest priority that was added first
	USoundMix* NextEQMix = NULL;
	FSoundMixState* NextState = NULL;

	for (decltype(SoundMixModifiers)::TIterator It(SoundMixModifiers); It; ++It)
	{
		if (It.Key() != IgnoredSoundMix && It.Value().CurrentState < ESoundMixState::FadingOut
			&& (NextEQMix == NULL
				|| (It.Key()->EQPriority > NextEQMix->EQPriority
					|| (It.Key()->EQPriority == NextEQMix->EQPriority
						&& It.Value().StartTime < NextState->StartTime)
					)
				)
			)
		{
			NextEQMix = It.Key();
			NextState = &(It.Value());
		}
	}

	return NextEQMix;
}

void FAudioDevice::ClearSoundMix(USoundMix* SoundMix)
{
	if (SoundMix == nullptr)
	{
		return;
	}

	if (SoundMix == BaseSoundMix)
	{
		BaseSoundMix = nullptr;
	}
	SoundMixModifiers.Remove(SoundMix);
	PrevPassiveSoundMixModifiers.Remove(SoundMix);

	// Check if there are any overrides for this sound mix and if so, reset them so that next time this sound mix is applied, it'll get the new override values
	FSoundMixClassOverrideMap* SoundMixOverrideMap = SoundMixClassEffectOverrides.Find(SoundMix);
	if (SoundMixOverrideMap)
	{
		for (TPair<USoundClass*, FSoundMixClassOverride>& Entry : *SoundMixOverrideMap)
		{
			Entry.Value.bOverrideApplied = false;
		}
	}

	TryClearingEQSoundMix(SoundMix);
}

/** Static helper function which handles setting and updating the sound class adjuster override */
static void UpdateClassAdjustorOverrideEntry(FSoundClassAdjuster& ClassAdjustor, FSoundMixClassOverride& ClassAdjusterOverride, float DeltaTime)
{
	// If we've already applied the override in a previous frame
	if (ClassAdjusterOverride.bOverrideApplied)
	{
		// If we've received a new override value since our last update, then just set the dynamic parameters to the new value
		// The dynamic parameter objects will automatically smoothly travel to the new target value from its current value in the given time
		if (ClassAdjusterOverride.bOverrideChanged)
		{
			ClassAdjusterOverride.PitchOverride.Set(ClassAdjusterOverride.SoundClassAdjustor.PitchAdjuster, ClassAdjusterOverride.FadeInTime);
			ClassAdjusterOverride.VolumeOverride.Set(ClassAdjusterOverride.SoundClassAdjustor.VolumeAdjuster, ClassAdjusterOverride.FadeInTime);
		}
		else
		{
			// We haven't changed so just update the override this frame
			ClassAdjusterOverride.PitchOverride.Update(DeltaTime);
			ClassAdjusterOverride.VolumeOverride.Update(DeltaTime);
		}
	}
	else
	{
		// We haven't yet applied the override to the mix, so set the override dynamic parameters to immediately
		// have the current class adjuster values (0.0 interp-time), then set the dynamic parameters to the new target values in the given fade time

		ClassAdjusterOverride.VolumeOverride.Set(ClassAdjustor.VolumeAdjuster, 0.0f);
		ClassAdjusterOverride.VolumeOverride.Set(ClassAdjusterOverride.SoundClassAdjustor.VolumeAdjuster, ClassAdjusterOverride.FadeInTime);

		ClassAdjusterOverride.PitchOverride.Set(ClassAdjustor.PitchAdjuster, 0.0f);
		ClassAdjusterOverride.PitchOverride.Set(ClassAdjusterOverride.SoundClassAdjustor.PitchAdjuster, ClassAdjusterOverride.FadeInTime);
	}

	if (!ClassAdjustor.SoundClassObject)
	{
		ClassAdjustor.SoundClassObject = ClassAdjusterOverride.SoundClassAdjustor.SoundClassObject;
	}

	check(ClassAdjustor.SoundClassObject == ClassAdjusterOverride.SoundClassAdjustor.SoundClassObject);

	// Get the current value of the dynamic parameters
	ClassAdjustor.PitchAdjuster = ClassAdjusterOverride.PitchOverride.GetValue();
	ClassAdjustor.VolumeAdjuster = ClassAdjusterOverride.VolumeOverride.GetValue();

	// Override the apply to children if applicable
	ClassAdjustor.bApplyToChildren = ClassAdjusterOverride.SoundClassAdjustor.bApplyToChildren;

	// Reset the flags on the override adjuster
	ClassAdjusterOverride.bOverrideApplied = true;
	ClassAdjusterOverride.bOverrideChanged = false;

	// Check if we're clearing and check the terminating condition
	if (ClassAdjusterOverride.bIsClearing)
	{
		// If our override dynamic parameter is done, then we've finished clearing
		if (ClassAdjusterOverride.VolumeOverride.IsDone())
		{
			ClassAdjusterOverride.bIsCleared = true;
		}
	}
}

float FAudioDevice::GetInterpolatedFrequency(const float InFrequency, const float InterpValue) const
{
	const float NormFrequency = InterpolateAdjuster(Audio::GetLinearFrequencyClamped(InFrequency, FVector2D(0.0f, 1.0f), FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY)), InterpValue);
	return Audio::GetLogFrequencyClamped(NormFrequency, FVector2D(0.0f, 1.0f), FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY));
}

void FAudioDevice::ApplyClassAdjusters(USoundMix* SoundMix, float InterpValue, float DeltaTime)
{
	if (!SoundMix)
	{
		return;
	}

	InterpValue = FMath::Clamp(InterpValue, 0.0f, 1.0f);

	// Check if there is a sound mix override entry
	FSoundMixClassOverrideMap* SoundMixOverrideMap = SoundMixClassEffectOverrides.Find(SoundMix);

	// Create a ptr to the array of sound class adjusters ers we want to actually use. Default to using the sound class effects adjuster list
	TArray<FSoundClassAdjuster>* SoundClassAdjusters = &SoundMix->SoundClassEffects;

	bool bUsingOverride = false;

	// If we have an override for this sound mix, replace any overrides and/or add to the array if the sound class adjustment entry doesn't exist
	if (SoundMixOverrideMap)
	{
		// If we have an override map, create a copy of the sound class adjusters for the sound mix, then override the sound mix class overrides
		SoundClassAdjustersCopy = SoundMix->SoundClassEffects;

		// Use the copied list
		SoundClassAdjusters = &SoundClassAdjustersCopy;

		bUsingOverride = true;

		// Get the interpolated values of the vanilla adjusters up-front
		for (FSoundClassAdjuster& Entry : *SoundClassAdjusters)
		{
			if (Entry.SoundClassObject)
			{
				Entry.VolumeAdjuster = InterpolateAdjuster(Entry.VolumeAdjuster, InterpValue);
				Entry.PitchAdjuster = InterpolateAdjuster(Entry.PitchAdjuster, InterpValue);
				Entry.VoiceCenterChannelVolumeAdjuster = InterpolateAdjuster(Entry.VoiceCenterChannelVolumeAdjuster, InterpValue);
				Entry.LowPassFilterFrequency = GetInterpolatedFrequency(Entry.LowPassFilterFrequency, InterpValue);
			}
		}

		TArray<USoundClass*> SoundClassesToRemove;
		for (TPair<USoundClass*, FSoundMixClassOverride>& SoundMixOverrideEntry : *SoundMixOverrideMap)
		{
			// Get the sound class object of the override
			FSoundMixClassOverride& ClassAdjusterOverride = SoundMixOverrideEntry.Value;
			USoundClass* SoundClassObject = ClassAdjusterOverride.SoundClassAdjustor.SoundClassObject;

			// If the override has successfully cleared, then just remove it and continue iterating
			if (ClassAdjusterOverride.bIsCleared)
			{
				SoundClassesToRemove.Add(SoundClassObject);
				continue;
			}

			// Look for it in the adjusters copy
			bool bSoundClassAdjustorExisted = false;
			for (FSoundClassAdjuster& Entry : *SoundClassAdjusters)
			{
				// If we found it, then we need to override the volume and pitch values of the adjuster entry
				if (Entry.SoundClassObject == SoundClassObject)
				{
					// Flag that we don't need to add it to the SoundClassAdjustorsCopy
					bSoundClassAdjustorExisted = true;

					UpdateClassAdjustorOverrideEntry(Entry, ClassAdjusterOverride, DeltaTime);
					break;
				}
			}

			// If we didn't find an existing sound class we need to add the override to the adjuster copy
			if (!bSoundClassAdjustorExisted)
			{
				// Create a default sound class adjuster (1.0 values for pitch and volume)
				FSoundClassAdjuster NewEntry;

				// Apply and/or update the override
				UpdateClassAdjustorOverrideEntry(NewEntry, ClassAdjusterOverride, DeltaTime);

				// Add the new sound class adjuster entry to the array
				SoundClassAdjusters->Add(NewEntry);
			}
		}

		for (USoundClass* SoundClassToRemove : SoundClassesToRemove)
		{
			SoundMixOverrideMap->Remove(SoundClassToRemove);

			// If there are no more overrides, remove the sound mix override entry
			if (SoundMixOverrideMap->Num() == 0)
			{
				SoundMixClassEffectOverrides.Remove(SoundMix);
			}
		}
	}

	// Loop through the sound class adjusters, everything should be up-to-date
	for (FSoundClassAdjuster& Entry : *SoundClassAdjusters)
	{
		if (Entry.SoundClassObject)
		{
			if (Entry.bApplyToChildren)
			{
				// If we're using the override, Entry will already have interpolated values
				if (bUsingOverride)
				{
					RecursiveApplyAdjuster(Entry, Entry.SoundClassObject);
				}
				else
				{
					// Copy the entry with the interpolated values before applying it recursively
					FSoundClassAdjuster EntryCopy = Entry;
					EntryCopy.VolumeAdjuster = InterpolateAdjuster(Entry.VolumeAdjuster, InterpValue);
					EntryCopy.PitchAdjuster = InterpolateAdjuster(Entry.PitchAdjuster, InterpValue);
					EntryCopy.VoiceCenterChannelVolumeAdjuster = InterpolateAdjuster(Entry.VoiceCenterChannelVolumeAdjuster, InterpValue);
					EntryCopy.LowPassFilterFrequency = GetInterpolatedFrequency(Entry.LowPassFilterFrequency, InterpValue);

					RecursiveApplyAdjuster(EntryCopy, Entry.SoundClassObject);
				}
			}
			else
			{
				// Apply the adjuster to only the sound class specified by the adjuster
				FSoundClassProperties* Properties = SoundClasses.Find(Entry.SoundClassObject);
				if (Properties)
				{
					// If we are using an override, we've already interpolated all our dynamic parameters
					if (bUsingOverride)
					{
						Properties->Volume *= Entry.VolumeAdjuster;
						Properties->Pitch *= Entry.PitchAdjuster;
						Properties->VoiceCenterChannelVolume *= Entry.VoiceCenterChannelVolumeAdjuster;

						if (Entry.LowPassFilterFrequency < Properties->LowPassFilterFrequency)
						{
							Properties->LowPassFilterFrequency = Entry.LowPassFilterFrequency;
						}
					}
					// Otherwise, we need to use the "static" data and compute the adjustment interpolations now
					else
					{
						Properties->Volume *= InterpolateAdjuster(Entry.VolumeAdjuster, InterpValue);
						Properties->Pitch *= InterpolateAdjuster(Entry.PitchAdjuster, InterpValue);
						Properties->VoiceCenterChannelVolume *= InterpolateAdjuster(Entry.VoiceCenterChannelVolumeAdjuster, InterpValue);

						const float NewLPF = GetInterpolatedFrequency(Entry.LowPassFilterFrequency, InterpValue);
						if (NewLPF < Properties->LowPassFilterFrequency)
						{
							Properties->LowPassFilterFrequency = NewLPF;
						}
					}
				}
				else
				{
					UE_LOGF(LogAudio, Warning, "Sound class '%ls' does not exist", *Entry.SoundClassObject->GetName());
				}
			}
		}
	}
}

void FAudioDevice::UpdateSoundClassProperties(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_UpdateSoundClasses);

	// Remove SoundMix modifications and propagate the properties down the hierarchy
	ParseSoundClasses(DeltaTime);

	TArray<TObjectPtr<USoundMix>, TInlineAllocator<32>> SoundMixesToClear;

	for (TPair<TObjectPtr<USoundMix>, FSoundMixState>& SoundMix :SoundMixModifiers)
	{
		FSoundMixState* SoundMixState = &SoundMix.Value;

		// Initial delay before mix is applied
		const double AudioTime = GetAudioClock();

		if (AudioTime >= SoundMixState->StartTime && AudioTime < SoundMixState->FadeInStartTime)
		{
			SoundMixState->InterpValue = 0.0f;
			SoundMixState->CurrentState = ESoundMixState::Inactive;
		}
		else if (AudioTime >= SoundMixState->FadeInStartTime && AudioTime < SoundMixState->FadeInEndTime)
		{
			// Work out the fade in portion
			SoundMixState->InterpValue = (float)((AudioTime - SoundMixState->FadeInStartTime) / (SoundMixState->FadeInEndTime - SoundMixState->FadeInStartTime));
			SoundMixState->CurrentState = ESoundMixState::FadingIn;
		}
		else if (AudioTime >= SoundMixState->FadeInEndTime
			&& (SoundMixState->IsBaseSoundMix
			|| ((SoundMixState->PassiveRefCount > 0 || SoundMixState->ActiveRefCount > 0) && SoundMixState->FadeOutStartTime < 0.f)
			|| AudioTime < SoundMixState->FadeOutStartTime))
		{
			// .. ensure the full mix is applied between the end of the fade in time and the start of the fade out time
			// or if SoundMix is the base or active via a passive push - ignores duration.
			SoundMixState->InterpValue = 1.0f;
			SoundMixState->CurrentState = ESoundMixState::Active;
		}
		else if (AudioTime >= SoundMixState->FadeOutStartTime && AudioTime < SoundMixState->EndTime)
		{
			// Work out the fade out portion
			SoundMixState->InterpValue = 1.0f - (float)((AudioTime - SoundMixState->FadeOutStartTime) / (SoundMixState->EndTime - SoundMixState->FadeOutStartTime));
			if (SoundMixState->CurrentState != ESoundMixState::FadingOut)
			{
				// Start fading EQ at same time
				SoundMixState->CurrentState = ESoundMixState::FadingOut;
				TryClearingEQSoundMix(SoundMix.Key);
			}
		}
		else
		{
			// Clear the effect of this SoundMix - may need to revisit for passive
			SoundMixState->InterpValue = 0.0f;
			SoundMixState->CurrentState = ESoundMixState::AwaitingRemoval;
		}

		ApplyClassAdjusters(SoundMix.Key, SoundMixState->InterpValue, DeltaTime);

		if (SoundMixState->CurrentState == ESoundMixState::AwaitingRemoval && SoundMixState->PassiveRefCount == 0)
		{
			SoundMixesToClear.Add(SoundMix.Key);
		}
	}

	for (USoundMix* SoundMix : SoundMixesToClear)
	{
		ClearSoundMix(SoundMix);
	}
}

void FAudioDevice::VirtualizeInactiveLoops()
{
	// Check if virtual loop system is enabled and don't push to virtual if disabled.
	if (!FAudioVirtualLoop::IsEnabled())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_VirtualizeLoops);

	const bool bDoRangeCheck = true;
	// Keep track of sounds to virtualize, then virtualize them after 
	// to prevent ActiveSound array from changing while iterating 
	// (ex. byOnAudioVirtualizationChanged delegate)
	TArray<FActiveSound*> ActiveSoundsToVirtualize;
	for (FActiveSound* ActiveSound : ActiveSounds)
	{
		// Don't virtualize if set to fade out
		if (ActiveSound->FadeOut != FActiveSound::EFadeOut::None)
		{
			continue;
		}

		// If already pending stop, don't attempt to virtualize
		if (IsPendingStop(ActiveSound))
		{
			continue;
		}

		if (!ActiveSound->Sound)
		{
			continue;
		}

		ActiveSoundsToVirtualize.Add(ActiveSound);
	}

	for (FActiveSound* ActiveSoundToVirtualize : ActiveSoundsToVirtualize)
	{
		FAudioVirtualLoop VirtualLoop;
		if (FAudioVirtualLoop::Virtualize(*ActiveSoundToVirtualize, bDoRangeCheck, VirtualLoop))
		{
			AddSoundToStop(ActiveSoundToVirtualize);

			// Clear must be called after AddSoundToStop to ensure AudioComponent is properly removed from AudioComponentIDToActiveSoundMap
			ActiveSoundToVirtualize->ClearAudioComponent();
			if (USoundBase* Sound = ActiveSoundToVirtualize->GetSound())
			{
				UE_LOGF(LogAudio, Verbose, "Playing ActiveSound %ls Virtualizing: Out of audible range.", *Sound->GetName());
			}
			AddVirtualLoop(VirtualLoop);
		}
	}
}

FVector FListener::GetPosition(bool bAllowOverride) const
{
	if (bAllowOverride && bUseAttenuationOverride)
	{
		return AttenuationOverride;
	}

	return Transform.GetTranslation();
}

float FListener::Interpolate(const double EndTime)
{
	if (FApp::GetCurrentTime() < InteriorStartTime)
	{
		return 0.0f;
	}

	if (FApp::GetCurrentTime() >= EndTime)
	{
		return 1.0f;
	}

	float InterpValue = (float)((FApp::GetCurrentTime() - InteriorStartTime) / (EndTime - InteriorStartTime));
	return FMath::Clamp(InterpValue, 0.0f, 1.0f);
}

void FListener::UpdateCurrentInteriorSettings()
{
	// Store the interpolation value, not the actual value
	InteriorVolumeInterp = Interpolate(InteriorEndTime);
	ExteriorVolumeInterp = Interpolate(ExteriorEndTime);
	InteriorLPFInterp = Interpolate(InteriorLPFEndTime);
	ExteriorLPFInterp = Interpolate(ExteriorLPFEndTime);
}

void FAudioDevice::InvalidateCachedInteriorVolumes() const
{
	check(IsInAudioThread());

	for (FActiveSound* ActiveSound : ActiveSounds)
	{
		ActiveSound->bGotInteriorSettings = false;
	}
}

void FListener::ApplyInteriorSettings(const uint32 InAudioVolumeID, const FInteriorSettings& Settings)
{
	if (InAudioVolumeID != AudioVolumeID || Settings != InteriorSettings)
	{
		// Use previous/ current interpolation time if we're transitioning to the default worldsettings zone.
		InteriorStartTime = FApp::GetCurrentTime();
		InteriorEndTime = InteriorStartTime + (Settings.bIsWorldSettings ? InteriorSettings.InteriorTime : Settings.InteriorTime);
		ExteriorEndTime = InteriorStartTime + (Settings.bIsWorldSettings ? InteriorSettings.ExteriorTime : Settings.ExteriorTime);
		InteriorLPFEndTime = InteriorStartTime + (Settings.bIsWorldSettings ? InteriorSettings.InteriorLPFTime : Settings.InteriorLPFTime);
		ExteriorLPFEndTime = InteriorStartTime + (Settings.bIsWorldSettings ? InteriorSettings.ExteriorLPFTime : Settings.ExteriorLPFTime);

		AudioVolumeID = InAudioVolumeID;
		InteriorSettings = Settings;
	}
}

FVector FListenerProxy::GetPosition(bool bAllowOverride) const
{
	if (bAllowOverride && bUseAttenuationOverride)
	{
		return AttenuationOverride;
	}

	return Transform.GetTranslation();
}

void FAudioDevice::SetListener(UWorld* World, const int32 InViewportIndex, const FTransform& ListenerTransform, const float InDeltaSeconds)
{
	check(IsInGameThread());

	uint32 WorldID = INDEX_NONE;

	if (World != nullptr)
	{
		WorldID = World->GetUniqueID();
	}

	// Initialize the plugin listeners if we haven't already. This needs to be done here since this is when we're
	// guaranteed to have a world ptr and we've already initialized the audio device.
	if (World)
	{
		if (!bPluginListenersInitialized)
		{
			InitializePluginListeners(World);
			bPluginListenersInitialized = true;
		}
		else 
		{
			// World change event triggered on change in world of existing listener.
			if (InViewportIndex < ListenerProxies.Num())
			{
				if (ListenerProxies[InViewportIndex].WorldID != WorldID)
				{
					NotifyPluginListenersWorldChanged(World);
				}
			}
		}
	}

	// The copy is done because FTransform doesn't work to pass by value on Win32
	FTransform ListenerTransformCopy = ListenerTransform;

	if (!ensureMsgf(ListenerTransformCopy.IsValid(), TEXT("Invalid listener transform provided to AudioDevice")))
	{
		// If we have a bad transform give it something functional if totally wrong
		ListenerTransformCopy = FTransform::Identity;
	}

	if (InViewportIndex >= ListenerProxies.Num())
	{
		ListenerProxies.AddDefaulted(InViewportIndex - ListenerProxies.Num() + 1);
	}

	ListenerProxies[InViewportIndex].Transform = ListenerTransformCopy;
	ListenerProxies[InViewportIndex].WorldID = WorldID;

	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetListener"), STAT_AudioSetListener, STATGROUP_AudioThreadCommands);

	if (World)
	{
		AUDIO_SPATIALIZATION_PLUGIN_LLM_SCOPE

		for (TAudioPluginListenerPtr PluginManager : PluginListeners)
		{
			PluginManager->OnTick(World, InViewportIndex, ListenerTransformCopy, InDeltaSeconds);
		}
	}

	FAudioThread::RunCommandOnAudioThread([this, WorldID, InViewportIndex, ListenerTransformCopy, InDeltaSeconds]()
	{
		// Broadcast to a 3rd party plugin listener observer if enabled
		for (TAudioPluginListenerPtr PluginManager : PluginListeners)
		{
			AUDIO_SPATIALIZATION_PLUGIN_LLM_SCOPE

			PluginManager->OnListenerUpdated(this, InViewportIndex, ListenerTransformCopy, InDeltaSeconds);
		}

		const int32 StartingListenerCount = Listeners.Num();

		TArray<FListener>& AudioThreadListeners = Listeners;
		if (InViewportIndex >= AudioThreadListeners.Num())
		{
			const int32 NumListenersToAdd = InViewportIndex - AudioThreadListeners.Num() + 1;
			for (int32 i = 0; i < NumListenersToAdd; ++i)
			{
				AudioThreadListeners.Add(FListener(this));

				// While we're going through the process of moving from raw listener access to access by index, 
				// we're going to store our current index inside the listener to help in deprecation and backwards compat.
				const int32 CurrentIndex = i + StartingListenerCount;
				if (ensure(AudioThreadListeners.IsValidIndex(CurrentIndex)))
				{
					AudioThreadListeners[CurrentIndex].ListenerIndex = CurrentIndex;
				}
			}
		}

		FListener& Listener = AudioThreadListeners[InViewportIndex];
		Listener.Velocity = InDeltaSeconds > 0.f ?
			(ListenerTransformCopy.GetTranslation() - Listener.Transform.GetTranslation()) / InDeltaSeconds
			: FVector::ZeroVector;

#if ENABLE_NAN_DIAGNOSTIC
		if (Listener.Velocity.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("FAudioDevice::SetListener has detected a NaN in Listener Velocity"));
		}
#endif
		const bool bShouldListenerForceUpdate = FAudioVirtualLoop::ShouldListenerMoveForceUpdate(Listener.Transform, ListenerTransformCopy);

		Listener.WorldID = WorldID;
		Listener.Transform = ListenerTransformCopy;

		if (bShouldListenerForceUpdate)
		{
			const bool bForceUpdate = true;
			UpdateVirtualLoops(bForceUpdate);
		}

		OnListenerUpdated(AudioThreadListeners);

	}, GET_STATID(STAT_AudioSetListener));
}

void FAudioDevice::SetListenerAttenuationOverride(int32 ListenerIndex, const FVector AttenuationPosition)
{
	if (ListenerIndex == INDEX_NONE)
	{
		return;
	}

	if (!IsInAudioThread())
	{
		if (ListenerIndex >= ListenerProxies.Num())
		{
			return;
		}

	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetListenerAttenuationOverride"), STAT_AudioSetListenerAttenuationOverride, STATGROUP_AudioThreadCommands);

		ListenerProxies[ListenerIndex].AttenuationOverride = AttenuationPosition;
		ListenerProxies[ListenerIndex].bUseAttenuationOverride = true;

	FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, ListenerIndex, AttenuationPosition]()
		{
			AudioDevice->SetListenerAttenuationOverride(ListenerIndex, AttenuationPosition);
		}, GET_STATID(STAT_AudioSetListenerAttenuationOverride));
	}
	else
	{
		if (ensureMsgf(ListenerIndex < Listeners.Num(), TEXT("Listener Index %u out of range of available Listeners!"), ListenerIndex))
		{
			FListener& Listener = Listeners[ListenerIndex];
			const bool bPrevAttenuationOverride = Listener.bUseAttenuationOverride;

			Listener.bUseAttenuationOverride = true;
			Listener.AttenuationOverride = AttenuationPosition;

			if (!bPrevAttenuationOverride)
			{
				UpdateVirtualLoops(true);
			}
		}
	}
}

void FAudioDevice::ClearListenerAttenuationOverride(int32 ListenerIndex)
{
	if (ListenerIndex == INDEX_NONE)
	{
		return;
	}

	if (!IsInAudioThread())
	{
		if (ListenerIndex >= ListenerProxies.Num())
	{
		return;
	}

	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ClearListenerAttenuationOverride"), STAT_AudioClearListenerAttenuationOverride, STATGROUP_AudioThreadCommands);

		ListenerProxies[ListenerIndex].AttenuationOverride = FVector::ZeroVector;
		ListenerProxies[ListenerIndex].bUseAttenuationOverride = false;

	FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, ListenerIndex]()
	{
			AudioDevice->ClearListenerAttenuationOverride(ListenerIndex);
	}, GET_STATID(STAT_AudioClearListenerAttenuationOverride));
	}
	else
	{
		if(ensureMsgf(ListenerIndex < Listeners.Num(), TEXT("Listener Index %u out of range of available Listeners!"), ListenerIndex))
		{
			FListener& Listener = Listeners[ListenerIndex];
			if (Listener.bUseAttenuationOverride)
			{
				Listener.bUseAttenuationOverride = false;
				UpdateVirtualLoops(true);
			}
		}
	}
}

bool FAudioDevice::GetDefaultAudioSettings(uint32 WorldID, FReverbSettings& OutReverbSettings, FInteriorSettings& OutInteriorSettings) const
{
	check(IsInAudioThread());

	const TPair<FReverbSettings, FInteriorSettings>* DefaultAudioSettings = WorldIDToDefaultAudioVolumeSettingsMap.Find(WorldID);
	if (DefaultAudioSettings)
	{
		OutReverbSettings = DefaultAudioSettings->Key;
		OutInteriorSettings = DefaultAudioSettings->Value;
		return true;
	}

	return false;
}

void FAudioDevice::SetDefaultAudioSettings(UWorld* World, const FReverbSettings& DefaultReverbSettings, const FInteriorSettings& DefaultInteriorSettings)
{
	check(IsInGameThread());

	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetDefaultAudioSettings"), STAT_AudioSetDefaultAudioSettings, STATGROUP_AudioThreadCommands);

	FAudioDevice* AudioDevice = this;
	const uint32 WorldID = World->GetUniqueID();
	FAudioThread::RunCommandOnAudioThread([AudioDevice, WorldID, DefaultReverbSettings, DefaultInteriorSettings]()
	{
		AudioDevice->WorldIDToDefaultAudioVolumeSettingsMap.Add(WorldID, TPair<FReverbSettings,FInteriorSettings>(DefaultReverbSettings,DefaultInteriorSettings));

	}, GET_STATID(STAT_AudioSetDefaultAudioSettings));
}

void FAudioDevice::ResetAudioVolumeProxyChangedState()
{
	for (TPair<uint32, FAudioVolumeProxy>& AudioVolumePair : AudioVolumeProxies)
	{
		FAudioVolumeProxy& Proxy = AudioVolumePair.Value;
		Proxy.bChanged = false;
	}
}

void FAudioDevice::GetAudioVolumeSettings(const uint32 WorldID, const FVector& Location, FAudioVolumeSettings& OutSettings) const
{
	check(IsInAudioThread());

	for (const TPair<uint32,FAudioVolumeProxy>& AudioVolumePair : AudioVolumeProxies)
	{
		const FAudioVolumeProxy& Proxy = AudioVolumePair.Value;
		if (Proxy.WorldID == WorldID)
		{
			FVector Dummy;
			float DistanceSqr = 0.f;
			if (Proxy.BodyInstance->GetSquaredDistanceToBody(Location, DistanceSqr, Dummy) && DistanceSqr == 0.f)
			{
				OutSettings.AudioVolumeID = Proxy.AudioVolumeID;
				OutSettings.Priority = Proxy.Priority;
				OutSettings.ReverbSettings = Proxy.ReverbSettings;
				OutSettings.InteriorSettings = Proxy.InteriorSettings;
				OutSettings.SubmixSendSettings = Proxy.SubmixSendSettings;
				OutSettings.SubmixOverrideSettings = Proxy.SubmixOverrideSettings;
				OutSettings.bChanged = Proxy.bChanged;
				return;
			}
		}
	}

	OutSettings.AudioVolumeID = 0;

	if (GetDefaultAudioSettings(WorldID, OutSettings.ReverbSettings, OutSettings.InteriorSettings))
	{
		OutSettings.SubmixSendSettings.Reset();
	}
}

void FAudioDevice::GatherInteriorData(FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) const
{
	SubsystemCollection.ForEachSubsystem<IActiveSoundUpdateInterface>([&ActiveSound, &ParseParams](IActiveSoundUpdateInterface* ActiveSoundUpdate)
	{
		ActiveSoundUpdate->GatherInteriorData(ActiveSound, ParseParams);
		return true;
	});
}

void FAudioDevice::ApplyInteriorSettings(FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) const
{
	SubsystemCollection.ForEachSubsystem<IActiveSoundUpdateInterface>([&ActiveSound, &ParseParams](IActiveSoundUpdateInterface* ActiveSoundUpdate)
	{
		ActiveSoundUpdate->ApplyInteriorSettings(ActiveSound, ParseParams);
		return true;
	});
}

void FAudioDevice::SubsystemsAdjustParseParams(FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) const
{
	SubsystemCollection.ForEachSubsystem<IActiveSoundUpdateInterface>([&ActiveSound, &ParseParams](IActiveSoundUpdateInterface* ActiveSoundUpdate)
	{
		ActiveSoundUpdate->SubsystemsAdjustParseParams(ActiveSound, ParseParams);
		return true;
	});
}

void FAudioDevice::NotifySubsystemsActiveSoundCreated(FActiveSound& ActiveSound) const
{
	SubsystemCollection.ForEachSubsystem<IActiveSoundUpdateInterface>([&ActiveSound](IActiveSoundUpdateInterface* ActiveSoundUpdate)
	{
		ActiveSoundUpdate->NotifyActiveSoundCreated(ActiveSound);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ActiveSoundUpdate->OnNotifyAddActiveSound(ActiveSound);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return true;
	});
}

void FAudioDevice::NotifySubsystemsActiveSoundDeleting(FActiveSound& ActiveSound) const
{
	SubsystemCollection.ForEachSubsystem<IActiveSoundUpdateInterface>([&ActiveSound](IActiveSoundUpdateInterface* ActiveSoundUpdate)
	{
		ActiveSoundUpdate->NotifyActiveSoundDeleting(ActiveSound);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ActiveSoundUpdate->OnNotifyPendingDelete(ActiveSound);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return true;
	});
}

void FAudioDevice::NotifySubsystemsVirtualizedSoundCreated(FActiveSound& ActiveSound) const
{
	SubsystemCollection.ForEachSubsystem<IActiveSoundUpdateInterface>([&ActiveSound](IActiveSoundUpdateInterface* ActiveSoundUpdate)
	{
		ActiveSoundUpdate->NotifyVirtualizedSoundCreated(ActiveSound);
		return true;
	});
}

void FAudioDevice::NotifySubsystemsVirtualizedSoundDeleting(FActiveSound& ActiveSound) const
{
	SubsystemCollection.ForEachSubsystem<IActiveSoundUpdateInterface>([&ActiveSound](IActiveSoundUpdateInterface* ActiveSoundUpdate)
	{
		ActiveSoundUpdate->NotifyVirtualizedSoundDeleting(ActiveSound);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ActiveSoundUpdate->OnNotifyPendingDelete(ActiveSound);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return true;
	});
}

void FAudioDevice::NotifySubsystemsSetOwner(FActiveSound& ActiveSound, const UObject* Owner) const
{
	// DEPRECATED in 5.7
	// Calling SubsystemCollection.ForEachSubsystem here is not safe because
	// it is on the Game Thread while all other calls are on the Audio Thread.
	// There would be risk of concurrent calls, which will corrupt SubsystemCollection bIsIterating.
}

void FAudioDevice::SetBaseSoundMix(USoundMix* NewMix)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetBaseSoundMix"), STAT_AudioSetBaseSoundMix, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, NewMix]()
		{
			AudioDevice->SetBaseSoundMix(NewMix);

		}, GET_STATID(STAT_AudioSetBaseSoundMix));

		return;
	}

	if (NewMix && NewMix != BaseSoundMix)
	{
		USoundMix* OldBaseSoundMix = BaseSoundMix;
		BaseSoundMix = NewMix;

		if (OldBaseSoundMix)
		{
			FSoundMixState* OldBaseState = SoundMixModifiers.Find(OldBaseSoundMix);
			check(OldBaseState);
			OldBaseState->IsBaseSoundMix = false;
			TryClearingSoundMix(OldBaseSoundMix, OldBaseState);
		}

		// Check whether this SoundMix is already active
		FSoundMixState* ExistingState = SoundMixModifiers.Find(NewMix);
		if (!ExistingState)
		{
			// First time this mix has been set - add it and setup mix modifications
			ExistingState = &SoundMixModifiers.Add(NewMix, FSoundMixState());

			// Setup SoundClass modifications
			ApplySoundMix(NewMix, ExistingState);

			// Use it to set EQ Settings, which will check its priority
			if (Effects)
			{
				Effects->SetMixSettings(NewMix);
			}
		}

		ExistingState->IsBaseSoundMix = true;
	}
}

void FAudioDevice::PushSoundMixModifier(USoundMix* SoundMix, bool bIsPassive, bool bIsRetrigger)
{
	if (SoundMix)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.PushSoundMixModifier"), STAT_AudioPushSoundMixModifier, STATGROUP_AudioThreadCommands);

			FAudioDevice* AudioDevice = this;
			FAudioThread::RunCommandOnAudioThread([AudioDevice, SoundMix, bIsPassive]()
			{
				AudioDevice->PushSoundMixModifier(SoundMix, bIsPassive);

			}, GET_STATID(STAT_AudioPushSoundMixModifier));

			return;
		}

		FSoundMixState* SoundMixState = SoundMixModifiers.Find(SoundMix);

		if (!SoundMixState)
		{
			// First time this mix has been pushed - add it and setup mix modifications
			SoundMixState = &SoundMixModifiers.Add(SoundMix, FSoundMixState());

			// Setup SoundClass modifications
			ApplySoundMix(SoundMix, SoundMixState);

			// Use it to set EQ Settings, which will check its priority
			if (Effects)
			{
				Effects->SetMixSettings(SoundMix);
			}
		}
		else
		{
			UpdateSoundMix(SoundMix, SoundMixState);
		}

		// Increase the relevant ref count - we know pointer exists by this point
		if (!bIsRetrigger)
		{
			if (bIsPassive)
			{
				SoundMixState->PassiveRefCount++;
			}
			else
			{
				SoundMixState->ActiveRefCount++;
			}
		}
	}
}

void FAudioDevice::SetSoundMixClassOverride(USoundMix* InSoundMix, USoundClass* InSoundClass, float Volume, float Pitch, float FadeInTime, bool bApplyToChildren)
{
	if (!InSoundMix || !InSoundClass)
	{
		return;
	}

	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetSoundMixClassOverride"), STAT_AudioSetSoundMixClassOverride, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, InSoundMix, InSoundClass, Volume, Pitch, FadeInTime, bApplyToChildren]()
		{
			AudioDevice->SetSoundMixClassOverride(InSoundMix, InSoundClass, Volume, Pitch, FadeInTime, bApplyToChildren);

		}, GET_STATID(STAT_AudioSetSoundMixClassOverride));

		return;
	}

	FSoundMixClassOverrideMap& SoundMixClassOverrideMap = SoundMixClassEffectOverrides.FindOrAdd(InSoundMix);

	// Check if we've already added this sound class override
	FSoundMixClassOverride* ClassOverride = SoundMixClassOverrideMap.Find(InSoundClass);
	if (ClassOverride)
	{
		// Override the values of the sound class override with the new values
		ClassOverride->SoundClassAdjustor.SoundClassObject = InSoundClass;
		ClassOverride->SoundClassAdjustor.VolumeAdjuster = Volume;
		ClassOverride->SoundClassAdjustor.PitchAdjuster = Pitch;
		ClassOverride->SoundClassAdjustor.bApplyToChildren = bApplyToChildren;

		// Flag that we've changed so that the update will interpolate to new values
		ClassOverride->bOverrideChanged = true;
		ClassOverride->bIsClearing = false;
		ClassOverride->FadeInTime = FadeInTime;
	}
	else
	{
		// Create a new override struct
		FSoundMixClassOverride NewClassOverride;
		NewClassOverride.SoundClassAdjustor.SoundClassObject = InSoundClass;
		NewClassOverride.SoundClassAdjustor.VolumeAdjuster = Volume;
		NewClassOverride.SoundClassAdjustor.PitchAdjuster = Pitch;
		NewClassOverride.SoundClassAdjustor.bApplyToChildren = bApplyToChildren;
		NewClassOverride.FadeInTime = FadeInTime;

		SoundMixClassOverrideMap.Add(InSoundClass, NewClassOverride);
	}
}

void FAudioDevice::ClearSoundMixClassOverride(USoundMix* InSoundMix, USoundClass* InSoundClass, float FadeOutTime)
{
	if (!InSoundMix || !InSoundClass)
	{
		return;
	}

	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ClearSoundMixClassOverride"), STAT_AudioClearSoundMixClassOverride, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, InSoundMix, InSoundClass, FadeOutTime]()
		{
			AudioDevice->ClearSoundMixClassOverride(InSoundMix, InSoundClass, FadeOutTime);

		}, GET_STATID(STAT_AudioClearSoundMixClassOverride));

		return;
	}

	// Get the sound mix class override map for the sound mix. If this doesn't exist, then nobody overrode the sound mix
	FSoundMixClassOverrideMap* SoundMixClassOverrideMap = SoundMixClassEffectOverrides.Find(InSoundMix);
	if (!SoundMixClassOverrideMap)
	{
		return;
	}

	// Get the sound class override. If this doesn't exist, then the sound class wasn't previously overridden.
	FSoundMixClassOverride* SoundClassOverride = SoundMixClassOverrideMap->Find(InSoundClass);
	if (!SoundClassOverride)
	{
		return;
	}

	// If the override is currently applied, then we need to "fade out" the override
	if (SoundClassOverride->bOverrideApplied)
	{
		// Get the new target values that sound mix would be if it weren't overridden.
		// If this was a pure add to the sound mix, then the target values will be 1.0f (i.e. not applied)
		float VolumeAdjuster = 1.0f;
		float PitchAdjuster = 1.0f;

		// Loop through the sound mix class adjusters and set the volume adjuster to the value that would be in the sound mix
		for (const FSoundClassAdjuster& Adjustor : InSoundMix->SoundClassEffects)
		{
			if (Adjustor.SoundClassObject == InSoundClass)
			{
				VolumeAdjuster = Adjustor.VolumeAdjuster;
				PitchAdjuster = Adjustor.PitchAdjuster;
				break;
			}
		}

		SoundClassOverride->bIsClearing = true;
		SoundClassOverride->bIsCleared = false;
		SoundClassOverride->bOverrideChanged = true;
		SoundClassOverride->FadeInTime = FadeOutTime;
		SoundClassOverride->SoundClassAdjustor.VolumeAdjuster = VolumeAdjuster;
		SoundClassOverride->SoundClassAdjustor.PitchAdjuster = PitchAdjuster;
	}
	else
	{
		// Otherwise, we just simply remove the sound class override in the sound class override map
		SoundMixClassOverrideMap->Remove(InSoundClass);

		// If there are no more overrides, remove the sound mix override entry
		if (!SoundMixClassOverrideMap->Num())
		{
			SoundMixClassEffectOverrides.Remove(InSoundMix);
		}
	}

}

void FAudioDevice::PopSoundMixModifier(USoundMix* SoundMix, bool bIsPassive)
{
	if (SoundMix)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.PopSoundMixModifier"), STAT_AudioPopSoundMixModifier, STATGROUP_AudioThreadCommands);

			FAudioDevice* AudioDevice = this;
			FAudioThread::RunCommandOnAudioThread([AudioDevice, SoundMix, bIsPassive]()
			{
				AudioDevice->PopSoundMixModifier(SoundMix, bIsPassive);

			}, GET_STATID(STAT_AudioPopSoundMixModifier));

			return;
		}

		FSoundMixState* SoundMixState = SoundMixModifiers.Find(SoundMix);

		if (SoundMixState)
		{
			if (bIsPassive && SoundMixState->PassiveRefCount > 0)
			{
				SoundMixState->PassiveRefCount--;
			}
			else if (!bIsPassive && SoundMixState->ActiveRefCount > 0)
			{
				SoundMixState->ActiveRefCount--;
			}

			TryClearingSoundMix(SoundMix, SoundMixState);
		}
	}
}

void FAudioDevice::ClearSoundMixModifier(USoundMix* SoundMix)
{
	if (SoundMix)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ClearSoundMixModifier"), STAT_AudioClearSoundMixModifier, STATGROUP_AudioThreadCommands);

			FAudioDevice* AudioDevice = this;
			FAudioThread::RunCommandOnAudioThread([AudioDevice, SoundMix]()
			{
				AudioDevice->ClearSoundMixModifier(SoundMix);

			}, GET_STATID(STAT_AudioClearSoundMixModifier));

			return;
		}

		FSoundMixState* SoundMixState = SoundMixModifiers.Find(SoundMix);

		if (SoundMixState)
		{
			SoundMixState->ActiveRefCount = 0;

			TryClearingSoundMix(SoundMix, SoundMixState);
		}
	}
}

void FAudioDevice::ClearSoundMixModifiers()
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ClearSoundMixModifiers"), STAT_AudioClearSoundMixModifiers, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice]()
		{
			AudioDevice->ClearSoundMixModifiers();

		}, GET_STATID(STAT_AudioClearSoundMixModifiers));

		return;
	}

	// Work on local data since ClearSoundMixModifier might modify SoundMixModifiers
	TArray<TObjectPtr<USoundMix>, TInlineAllocator<32>> Keys;
	SoundMixModifiers.GenerateKeyArray(Keys);

	// Clear all sound mix modifiers
	for (USoundMix* Key : Keys)
	{
		ClearSoundMixModifier(Key);
	}
}

void FAudioDevice::ActivateReverbEffect(UReverbEffect* ReverbEffect, FName TagName, float Priority, float Volume, float FadeTime)
{
	check(IsInGameThread());

	FActivatedReverb& ActivatedReverb = ActivatedReverbs.FindOrAdd(TagName);

	ActivatedReverb.ReverbSettings.ReverbEffect = ReverbEffect;
	ActivatedReverb.ReverbSettings.Volume = Volume;
	ActivatedReverb.ReverbSettings.FadeTime = FadeTime;
	ActivatedReverb.Priority = Priority;

	UpdateHighestPriorityReverb();
}

void FAudioDevice::DeactivateReverbEffect(FName TagName)
{
	check(IsInGameThread());

	if (ActivatedReverbs.Remove(TagName) > 0)
	{
		UpdateHighestPriorityReverb();
	}
}

void* FAudioDevice::InitEffect(FSoundSource* Source)
{
	check(IsInAudioThread());
	if (Effects)
	{
		return Effects->InitEffect(Source);
	}
	return nullptr;
}

void* FAudioDevice::UpdateEffect(FSoundSource* Source)
{
	SCOPE_CYCLE_COUNTER(STAT_AudioUpdateEffects);

	check(IsInAudioThread());
	if (Effects)
	{
		return Effects->UpdateEffect(Source);
	}
	return nullptr;
}

void FAudioDevice::DestroyEffect(FSoundSource* Source)
{
	check(IsInAudioThread());
	if (Effects)
	{
		Effects->DestroyEffect(Source);
	}
}

void FAudioDevice::HandlePause(bool bGameTicking, bool bGlobalPause)
{
	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.HandlePause"), STAT_AudioHandlePause, STATGROUP_AudioThreadCommands);

	// Run this command on the audio thread if this is getting called on game thread
	if (!IsInAudioThread())
	{
		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, bGameTicking, bGlobalPause]()
		{
			AudioDevice->HandlePause(bGameTicking, bGlobalPause);
		}, GET_STATID(STAT_AudioHandlePause));

		return;
	}

	// Handles the global pause/unpause feature

	// Pause all sounds if transitioning to pause mode.
	if (!bGameTicking && (bGameWasTicking || bGlobalPause))
	{
		for (int32 i = 0; i < Sources.Num(); i++)
		{
			FSoundSource* Source = Sources[ i ];
			if (!Source->IsPausedByGame() && (bGlobalPause || Source->IsGameOnly()))
			{
				Source->SetPauseByGame(true);
			}
		}
	}
	// Unpause all sounds if transitioning back to game.
	else if (bGameTicking && (!bGameWasTicking || bGlobalPause))
	{
		for (int32 i = 0; i < Sources.Num(); i++)
		{
			FSoundSource* Source = Sources[ i ];
			if (Source->IsPausedByGame() && (bGlobalPause || Source->IsGameOnly()))
			{
				Source->SetPauseByGame(false);
			}
		}
	}

	bGameWasTicking = bGameTicking;
}

int32 FAudioDevice::GetSortedActiveWaveInstances(TArray<FWaveInstance*>& WaveInstances, const ESortedActiveWaveGetType::Type GetType)
{
	check(IsInAudioThread());

	SCOPE_CYCLE_COUNTER(STAT_AudioGatherWaveInstances);

	// Tick all the active audio components.  Use a copy as some operations may remove elements from the list, but we want
	// to evaluate in the order they were added
	TArray<FActiveSound*> ActiveSoundsCopy = ActiveSounds;
	for (int32 i = 0; i < ActiveSoundsCopy.Num(); ++i)
	{
		FActiveSound* ActiveSound = ActiveSoundsCopy[i];

		if (!ActiveSound)
		{
			UE_LOGF(LogAudio, Error, "Null sound at index %d in ActiveSounds Array!", i);
			continue;
		}

		if (!ActiveSound->Sound)
		{
			// No sound - cleanup and remove
			AddSoundToStop(ActiveSound);
		}
		// If the world scene allows audio - tick wave instances.
		else
		{
			UWorld* ActiveSoundWorldPtr = ActiveSound->World.Get();
			if (ActiveSoundWorldPtr == nullptr || ActiveSoundWorldPtr->AllowAudioPlayback())
			{
				bool bStopped = false;

				if (ActiveSound->IsOneShot() && !ActiveSound->bIsPreviewSound)
				{
					// Don't stop a sound if it's playing effect chain tails or has effects playing, active sound will stop on its own in this case (in audio mixer).
					USoundEffectSourcePresetChain* ActiveSourceEffectChain = ActiveSound->GetSourceEffectChain();
					if (!ActiveSourceEffectChain || !ActiveSourceEffectChain->bPlayEffectChainTails || ActiveSourceEffectChain->Chain.Num() == 0)
					{
						const float Duration = ActiveSound->Sound->GetDuration();
						if ((ActiveSound->Sound->HasDelayNode() || ActiveSound->Sound->HasConcatenatorNode()))
						{
							static const float TimeFudgeFactor = 1.0f;
							if (Duration > TimeFudgeFactor && ActiveSound->PlaybackTime > Duration + TimeFudgeFactor)
							{
								bStopped = true;
							}
						}
						else if (!ActiveSound->bIsPlayingAudio && ActiveSound->bFinished)
						{
							bStopped = true;
						}

						if (bStopped)
						{
							UE_LOGF(LogAudio, Log, "One-shot active sound stopped due to duration or because it didn't generate any audio: %g > %g : %ls %ls",
								ActiveSound->PlaybackTime,
								Duration,
								*ActiveSound->Sound->GetName(),
								*ActiveSound->GetAudioComponentName());

							AddSoundToStop(ActiveSound);
						}
					}
				}

				if (!bStopped)
				{
					// If not in game, do not advance sounds unless they are UI sounds.
					float UsedDeltaTime = GetGameDeltaTime();
					if (GetType == ESortedActiveWaveGetType::QueryOnly || (GetType == ESortedActiveWaveGetType::PausedUpdate && !ActiveSound->bIsUISound))
					{
						UsedDeltaTime = 0.0f;
					}

					ActiveSound->CacheClosestListener();
					ActiveSound->UpdateWaveInstances(WaveInstances, UsedDeltaTime);
					ActiveSound->UpdateInterfaceParameters(Listeners);
				}
			}
		}
	}

	if (GetType != ESortedActiveWaveGetType::QueryOnly)
	{
		UpdateConcurrency(WaveInstances, ActiveSoundsCopy);
	}

	int32 FirstActiveIndex = 0;
	// Only need to do the wave instance sort if we have any waves and if our wave instances are greater than our max channels.
	if (WaveInstances.Num() > 0)
	{
		// Helper function for "Sort" (higher priority sorts last).
		struct FCompareFWaveInstanceByPlayPriority
		{
			FORCEINLINE bool operator()(const FWaveInstance& A, const FWaveInstance& B) const
			{
				return A.GetVolumeWeightedPriority() < B.GetVolumeWeightedPriority();
			}
		};

		// Sort by priority (lowest priority first).
		WaveInstances.Sort(FCompareFWaveInstanceByPlayPriority());


		if (EnableRelativeRenderCostVoiceLimitCVar)
		{
			// Find the first active index based on the relative render cost estimate of the wave instances
			float RemainingRenderCost = (float)GetMaxChannels();

			FirstActiveIndex = WaveInstances.Num();
			for (const FWaveInstance* WaveInstance : ReverseIterate(WaveInstances))
			{
				RemainingRenderCost -= WaveInstance->GetRelativeRenderCost();
				// Do not increment index if we've gone past our max render cost. 
				if (RemainingRenderCost >= 0.f)
				{
					FirstActiveIndex--;
				}
				else
				{
					break;
				}
			}
		}
		else
		{
			// Get the first index that will result in a active source voice
			int32 CurrentMaxChannels = GetMaxChannels();
			FirstActiveIndex = FMath::Max(WaveInstances.Num() - CurrentMaxChannels, 0);
		}
	}

	return FirstActiveIndex;
}

void FAudioDevice::UpdateActiveSoundPlaybackTime(bool bIsGameTicking)
{
#if UE_AUDIO_PROFILERTRACE_ENABLED
	auto ShouldSendIsAliveTraceMessage = [](const FActiveSound& ActiveSound, const float DeltaTime) -> bool
	{
		if (ActiveSound.IsPlayingAudio())
		{
			// Send a ping to Audio Insights every second to confirm the sound is still playing.
			// This is to ensure long duration sounds (such as ambiences) will still appear in snapshot files if their start action was before the snapshot start
			return FMath::Floor(ActiveSound.PlaybackTimeUnscaled + DeltaTime) - FMath::Floor(ActiveSound.PlaybackTimeUnscaled) > 0.0f;
		}

		return false;
	};
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	if (bIsGameTicking)
	{
		const float DeltaTime = GetDeviceDeltaTime();

		for (FActiveSound* ActiveSound : ActiveSounds)
		{
#if UE_AUDIO_PROFILERTRACE_ENABLED
			if (ShouldSendIsAliveTraceMessage(*ActiveSound, DeltaTime))
			{
				SendSoundIsAlivePingTraceMessage(*ActiveSound);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			// Scale the playback time with the device delta time and the current "min pitch" of the sounds which would play on it.
			const float DeltaTimePitchCorrected = DeltaTime * ActiveSound->MinCurrentPitch;

			ActiveSound->PlaybackTime += DeltaTimePitchCorrected;
			ActiveSound->PlaybackTimeUnscaled += DeltaTime;
			ActiveSound->PlaybackTimeNonVirtualized += DeltaTimePitchCorrected;
		}
	}
	else if (GIsEditor)
	{
		const float DeltaTime = GetDeviceDeltaTime();

#if UE_AUDIO_PROFILERTRACE_ENABLED && WITH_EDITOR
		SoundIsAlivePingTimeSec += DeltaTime;

		const bool bSendIsAlivePing = SoundIsAlivePingTimeSec >= SoundIsAlivePingWaitTimeSec;

		if (bSendIsAlivePing)
		{
			SoundIsAlivePingTimeSec = 0.0f;
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED && WITH_EDITOR

		for (FActiveSound* ActiveSound : ActiveSounds)
		{
			if (ActiveSound->bIsPreviewSound)
			{
#if UE_AUDIO_PROFILERTRACE_ENABLED
				if (ShouldSendIsAliveTraceMessage(*ActiveSound, DeltaTime))
				{
					SendSoundIsAlivePingTraceMessage(*ActiveSound);
				}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				// Scale the playback time with the device delta time and the current "min pitch" of the sounds which would play on it.
				const float DeltaTimePitchCorrected = DeltaTime * ActiveSound->MinCurrentPitch;
				ActiveSound->PlaybackTime += DeltaTimePitchCorrected;
				ActiveSound->PlaybackTimeUnscaled += DeltaTime;
				ActiveSound->PlaybackTimeNonVirtualized += DeltaTimePitchCorrected;
			}
			else
			{
#if UE_AUDIO_PROFILERTRACE_ENABLED && WITH_EDITOR
				if (bSendIsAlivePing && ActiveSound->IsPlayingAudio())
				{
					SendSoundIsAlivePingTraceMessage(*ActiveSound);
				}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED && WITH_EDITOR
			}
		}
	}

}

void FAudioDevice::StopOldestStoppingSource()
{
	check(!FreeSources.Num());

	FSoundSource* LowestPriStoppingSource = nullptr;
	FSoundSource* LowestPriSource = nullptr;
	FSoundSource* LowestPriNonLoopingSource = nullptr;

	for (FSoundSource* Source : Sources)
	{
		// Find oldest stopping voice first
		if (Source->IsStopping())
		{
			if (!LowestPriStoppingSource)
			{
				LowestPriStoppingSource = Source;
			}
			else
			{
				if (Source->WaveInstance->GetVolumeWeightedPriority() < LowestPriStoppingSource->WaveInstance->GetVolumeWeightedPriority())
				{
					LowestPriStoppingSource = Source;
				}
			}
		}
		else if (Source->WaveInstance)
		{
			// Find lowest volume/priority non-looping source as fallback
			if (Source->WaveInstance->LoopingMode != ELoopingMode::LOOP_Forever && !Source->WaveInstance->bIsUISound)
			{
				if (!LowestPriNonLoopingSource)
				{
					LowestPriNonLoopingSource = Source;
				}
				else
				{
					if (Source->WaveInstance->GetVolumeWeightedPriority() < LowestPriNonLoopingSource->WaveInstance->GetVolumeWeightedPriority())
					{
						LowestPriNonLoopingSource = Source;
					}
				}
			}


			// Find lowest volume/priority source as final fallback
			if (!LowestPriSource)
			{
				LowestPriSource = Source;
			}
			else
			{
				if (Source->WaveInstance->GetVolumeWeightedPriority() < LowestPriSource->WaveInstance->GetVolumeWeightedPriority())
				{
					LowestPriSource = Source;
				}
			}
		}
	}

	// Stop oldest stopping source
	if (LowestPriStoppingSource)
	{
		LowestPriStoppingSource->StopNow();
	}
	// If no oldest stopping source, stop oldest one-shot
	else if (LowestPriNonLoopingSource)
	{
		LowestPriNonLoopingSource->StopNow();
	}
	// Otherwise stop oldest source.
	else
	{
		check(LowestPriSource);
		LowestPriSource->StopNow();
	}
	check(FreeSources.Num() > 0);
}

void FAudioDevice::StopSources(TArray<FWaveInstance*>& WaveInstances, int32 FirstActiveIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_StopSources);

	for (int32 InstanceIndex = FirstActiveIndex; InstanceIndex < WaveInstances.Num(); InstanceIndex++)
	{
		FWaveInstance& WaveInstance = *WaveInstances[InstanceIndex];

		// Flag active sounds that generated wave instances that they are trying to actively play audio now
		// This will avoid stopping one-shot active sounds that failed to generate audio this audio thread frame tick
		WaveInstance.ActiveSound->bIsPlayingAudio = true;

		// Touch sources that are high enough priority to play
		if (FSoundSource* Source = WaveInstanceSourceMap.FindRef(&WaveInstance))
		{
			Source->LastUpdate = CurrentTick;

			// If they are still audible, mark them as such
			float VolumeWeightedPriority = WaveInstance.GetVolumeWithDistanceAndOcclusionAttenuation() * WaveInstance.GetDynamicVolume();
			if (VolumeWeightedPriority > 0.0f)
			{
				Source->LastHeardUpdate = CurrentTick;
			}
		}
	}

	// Stop inactive sources, sources that no longer have a WaveInstance associated
	// or sources that need to be reset because Stop & Play were called in the same frame.
	for (int32 SourceIndex = 0; SourceIndex < Sources.Num(); SourceIndex++)
	{
		FSoundSource* Source = Sources[SourceIndex];

		if (FWaveInstance* WaveInstance = Source->WaveInstance)
		{
			// If we need to stop this sound due to max concurrency (i.e. it was quietest in a concurrency group)
			if (WaveInstance->ShouldStopDueToMaxConcurrency() || Source->LastUpdate != CurrentTick)
			{
				if (!Source->IsStopping())
				{
					Source->Stop();
				}
				else
				{
					// Still do update even if stopping
					Source->Update();
				}
			}
			else
			{
				// Update the pause state of the source.
				Source->SetPauseManually(WaveInstance->bIsPaused);

				// Have to check it again here, because:
				// - Source->NotifyPlaybackData() does not handle it if it is nullptr
				// - SetPauseManually might set it to nullptr if the sound is unpaused and is stopping
				if (Source->WaveInstance != nullptr)
				{
					// Need to update the source still so that it gets any volume settings applied to
					// otherwise the source may play at a very quiet volume and not actually set to 0.0
					Source->NotifyPlaybackData();
					Source->Update();
				}
			}

#if ENABLE_AUDIO_DEBUG
			Audio::FAudioDebugger::DrawDebugInfo(*Source);
#endif // ENABLE_AUDIO_DEBUG
		}
	}

	// Stop wave instances that are no longer playing due to priority reasons. This needs to happen AFTER
	// stopping sources as calling Stop on a sound source in turn notifies the wave instance of a buffer
	// being finished which might reset it being finished.
	for (int32 InstanceIndex = 0; InstanceIndex < FirstActiveIndex; InstanceIndex++)
	{
		if (FWaveInstance* WaveInstance = WaveInstances[InstanceIndex])
		{
			WaveInstance->StopWithoutNotification();
		}
	}

#if ENABLE_AUDIO_DEBUG
	Audio::FAudioDebugger::UpdateAudibleInactiveSounds(FirstActiveIndex, WaveInstances);
#endif // ENABLE_AUDIO_DEBUG
}

void FAudioDevice::StartSources(TArray<FWaveInstance*>& WaveInstances, int32 FirstActiveIndex, bool bGameTicking)
{
	check(IsInAudioThread());

	SCOPE_CYCLE_COUNTER(STAT_AudioStartSources);
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_StartSources);

	TArray<USoundWave*> StartingSoundWaves;

	// Start sources as needed.
	for (int32 InstanceIndex = FirstActiveIndex; InstanceIndex < WaveInstances.Num(); InstanceIndex++)
	{
		FWaveInstance* WaveInstance = WaveInstances[InstanceIndex];

		USoundWave* WaveData = WaveInstance->WaveData;
		if (!WaveData)
		{
			continue;
		}

		// Make sure we've finished precaching the wave instance's wave data before trying to create a source for it
		ESoundWavePrecacheState PrecacheState = WaveData->GetPrecacheState();
		const bool bIsSoundWaveStillLoading = WaveData->HasAnyFlags(RF_NeedLoad);
		if (PrecacheState == ESoundWavePrecacheState::InProgress || (WaitForSoundWaveToLoadCvar && bIsSoundWaveStillLoading))
		{
			continue;
		}

		// Editor uses bIsUISound for sounds played in the browser.
		if (!WaveInstance->ShouldStopDueToMaxConcurrency() && (bGameTicking || WaveInstance->bIsUISound))
		{
			FSoundSource* Source = WaveInstanceSourceMap.FindRef(WaveInstance);
			if (!Source)
			{
				// Check for full sources and stop the oldest stopping source
				if (!FreeSources.Num())
				{
					StopOldestStoppingSource();
				}

				check(FreeSources.Num());
				Source = FreeSources.Pop();
				check(Source);

				StartingSoundWaves.AddUnique(WaveData);

				// Prepare for initialization...
				bool bSuccess = false;
				if (Source->PrepareForInitialization(WaveInstance))
				{
					// We successfully prepared for initialization (though we may not be prepared to actually init yet)
					bSuccess = true;

					// If we are now prepared to init (because the file handle and header synchronously loaded), then init right away
					if (Source->IsPreparedToInit())
					{
						// Init the source, this may result in failure
						bSuccess = Source->Init(WaveInstance);

						// If we succeeded then play and update the source
						if (bSuccess)
						{
							// Set the pause before updating it
							Source->SetPauseManually(Source->WaveInstance->bIsPaused);

							check(Source->IsInitialized());
							Source->Update();

							// If the source didn't get paused while initializing, then play it
							if (!Source->IsPaused())
							{
								Source->Play();
							}
						}
					}
					else
					{
						// This sound is not yet prepared to play, perform any necessary operations 
						UpdateUnpreparedSound(WaveInstance, bGameTicking);
					}
				}

				// If we succeeded above then we need to map the wave instance to the source
				if (bSuccess)
				{
					// Associate wave instance with it which is used earlier in this function.
					WaveInstanceSourceMap.Add(WaveInstance, Source);
				}
				else
				{
					// If we failed, then we need to stop the wave instance and add the source back to the free list
					// This can happen if e.g. the USoundWave pointed to by the WaveInstance is not a valid sound file.
					// If we don't stop the wave file, it will continue to try initializing the file every frame, which is a perf hit
					UE_LOGF(LogAudio, Log, "Failed to start sound source for %ls", (WaveInstance->ActiveSound && WaveInstance->ActiveSound->Sound) ? *WaveInstance->ActiveSound->Sound->GetName() : TEXT("UNKNOWN") );
					AddSoundToStop(WaveInstance->ActiveSound);
					WaveInstance->StopWithoutNotification();
					Source->WaveInstance = nullptr;
					FreeSources.Add(Source);
					WaveInstanceSourceMap.Remove(WaveInstance);
				}
			}
			else // Source is valid
			{
				if (!Source->IsInitialized() && Source->IsPreparedToInit())
				{
					// Try to initialize the source. This may fail if something is wrong with the source.
					if (Source->Init(WaveInstance))
					{
						Source->Update();

						// Note: if we succeeded in starting to prepare to init, we already added the wave instance map to the source so don't need to add here.
						check(Source->IsInitialized());

						// If the source didn't get paused while initializing, then play it
						if (!Source->IsPaused())
						{
							Source->Play();
						}
					}
					else
					{
						// If were ready to call init but failed, then we need to add the source and stop with notification
						WaveInstance->StopWithoutNotification();
						FreeSources.Add(Source);
					}
				}
			}
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FGameThreadAudioTask.AddReferencedSoundWaves"), STAT_AudioAddReferencedSoundWaves, STATGROUP_TaskGraphTasks);
	
	// Run a command to make sure we add the starting sounds to the referenced sound waves list
	if (StartingSoundWaves.Num() > 0)
	{
		FScopeLock AudioThreadReferenceLock(&AudioThreadReferenceCritSec);

		for (USoundWave* SoundWave : StartingSoundWaves)
		{
			ReferencedSoundWaves_AudioThread.AddUnique(SoundWave);
		}
	}
}

void FAudioDevice::UpdateReferencedSoundWaves()
{
	{
		FScopeLock AudioThreadReferenceLock(&AudioThreadReferenceCritSec);

		for (USoundWave* SoundWave : ReferencedSoundWaves_AudioThread)
		{
			ReferencedSoundWaves.AddUnique(SoundWave);
		}

		ReferencedSoundWaves_AudioThread.Reset();
	}

	// On game thread, look through registered sound waves and remove if we finished precaching (and audio decompressor is cleaned up)
	// ReferencedSoundWaves is used to make sure GC doesn't run on any sound waves that are actively pre-caching within an async task.
	// Sounds may be loaded, kick off an async task to decompress, but never actually try to play, so GC can reclaim these while precaches are in-flight.
	// We are also tracking when a sound wave is actively being used to generate audio in the audio render to prevent GC from happening to sounds till being used in the audio renderer.
	for (int32 i = ReferencedSoundWaves.Num() - 1; i >= 0; --i)
	{
		USoundWave* Wave = ReferencedSoundWaves[i];
		bool bRemove = true;
		// If this is null that means it was nulled out in AddReferencedObjects via mark pending kill
		if (Wave)
		{
			const bool bIsPrecacheDone = (Wave->GetPrecacheState() == ESoundWavePrecacheState::Done);
			const bool bIsGeneratingAudio = Wave->IsGeneratingAudio();

			if (!bIsPrecacheDone || bIsGeneratingAudio)
			{
				bRemove = false;
			}
		}

		if (bRemove)
		{
			ReferencedSoundWaves.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}
}

void FAudioDevice::Update(bool bGameTicking)
{
	LLM_SCOPE(ELLMTag::AudioMisc);

	if (IsInGameThread())
	{
		UpdateGameThread();

		// Make sure our referenced sound waves is up-to-date
		UpdateReferencedSoundWaves();
	}

	if (!IsInAudioThread())
	{
		check(IsInGameThread());

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, bGameTicking]()
		{
			AudioDevice->Update(bGameTicking);
		});

		// We process all enqueued commands on the audio device update
		FAudioThread::ProcessAllCommands();

		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevic_Update);

	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AudioUpdateTime"), STAT_AudioUpdateTime, STATGROUP_AudioThreadCommands);
	FScopeCycleCounter AudioUpdateTimeCounter(GET_STATID(STAT_AudioUpdateTime));

	// On audio thread, look through precaching sound waves and remove if we finished task and clean it up.
	// Note we can only touch the precache async task from the audio thread so must clean it up here.
	for (int32 i = PrecachingSoundWaves.Num() - 1; i >= 0; --i)
	{
		USoundWave* Wave = PrecachingSoundWaves[i];
		if (Wave->CleanupDecompressor())
		{
			PrecachingSoundWaves.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}

	bIsStoppingVoicesEnabled = !DisableStoppingVoicesCvar;

	// Update the master volume
	PrimaryVolume = GetTransientPrimaryVolume();
	
	if (!DisableAppVolumeCvar)
	{
		PrimaryVolume *= FApp::GetVolumeMultiplier();
	}


	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_UpdateDeviceTiming);

		// Updates hardware timing logic. Only implemented in audio mixer.
		UpdateHardwareTiming();

		// Updates the audio device delta time
		UpdateDeviceDeltaTime();
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_UpdateVirtualLoops);
		// Update which loops should re-trigger due to coming back into proximity
		// or allowed by concurrency re-evaluating in context of other sounds stopping
		const bool bForceUpdate = false;
		UpdateVirtualLoops(bForceUpdate);
	}

	// update if baked analysis is enabled
	bIsBakedAnalysisEnabled = (BakedAnalysisEnabledCVar == 1);

	if (bGameTicking)
	{
		GlobalPitchScale.Update(GetDeviceDeltaTime());
	}

	// Start a new frame
	CurrentTick++;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_HandlePause);

		// Handle pause/unpause for the game and editor.
		HandlePause(bGameTicking);
	}

	UpdateAudioVolumeEffects();

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_UpdateAudioEngineSubsystems);

		// Updates our audio engine subsystems 
		UpdateAudioEngineSubsystems();
	}

#if ENABLE_AUDIO_DEBUG
	if (GEngine)
	{
		if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
		{
			TArray<UWorld*> Worlds = DeviceManager->GetWorldsUsingAudioDevice(DeviceID);
			for (UWorld* World : Worlds)
			{
				if (World)
				{
					Audio::FAudioDebugger::DrawDebugInfo(*World, Listeners);
				}
			}
		}
	}
#endif // ENABLE_AUDIO_DEBUG

	// Gets the current state of the sound classes accounting for sound mix
	UpdateSoundClassProperties(GetDeviceDeltaTime());

	// Set looping ActiveSounds that are out-of-range to virtual and add to stop
	VirtualizeInactiveLoops();

	ProcessingPendingActiveSoundStops();

	// Update listener transforms
	if (Listeners.Num() != InverseListenerTransforms.Num())
	{
		InverseListenerTransforms.SetNum(Listeners.Num());
	}

	for (int32 ListenerIndex = 0; ListenerIndex < Listeners.Num(); ++ListenerIndex)
	{
		// Caches the matrix used to transform a sounds position into local space so we can just look
		// at the Y component after normalization to determine spatialization.
		const FListener& Listener = Listeners[ListenerIndex];
		FMatrix& InverseTransform = InverseListenerTransforms[ListenerIndex];

		const FVector Up = Listener.GetUp();
		const FVector Right = Listener.GetFront();
		InverseTransform = FMatrix(Up, Right, Up ^ Right, Listener.Transform.GetTranslation()).Inverse();
		ensure(!InverseTransform.ContainsNaN());
	}

	int32 FirstActiveIndex = INDEX_NONE;

	if (Sources.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_UpdateSources);

		// Kill any sources that have finished
		for (int32 SourceIndex = 0; SourceIndex < Sources.Num(); SourceIndex++)
		{
			// Source has finished playing (it's one shot)
			if (Sources[ SourceIndex ]->IsFinished())
			{
				Sources[ SourceIndex ]->Stop();
			}
		}

		// Poll audio components for active wave instances (== paths in node tree that end in a USoundWave)
		ActiveWaveInstances.Reset();
		FirstActiveIndex = GetSortedActiveWaveInstances(ActiveWaveInstances, (bGameTicking ? ESortedActiveWaveGetType::FullUpdate : ESortedActiveWaveGetType::PausedUpdate));

		// Stop sources that need to be stopped, and touch the ones that need to be kept alive
		StopSources(ActiveWaveInstances, FirstActiveIndex);

		// Start and/or update any sources that have a high enough priority to play
		StartSources(ActiveWaveInstances, FirstActiveIndex, bGameTicking);

		// Check which sounds are active from these wave instances and update passive SoundMixes
		UpdatePassiveSoundMixModifiers(ActiveWaveInstances, FirstActiveIndex);

		// If not paused, update the playback time of the active sounds after we've processed passive mix modifiers
		// Note that for sounds which play while paused, this will result in longer active sound playback times, which will be ok. If we update the
		// active sound is updated while paused (for a long time), most sounds will be stopped when unpaused.
		UpdateActiveSoundPlaybackTime(bGameTicking);

		const int32 Channels = GetMaxChannels();
		SET_DWORD_STAT(STAT_WaveInstances, ActiveWaveInstances.Num());
		SET_DWORD_STAT(STAT_AudioSources, Sources.Num() - FreeSources.Num());
		SET_DWORD_STAT(STAT_WavesDroppedDueToPriority, FMath::Max(ActiveWaveInstances.Num() - Sources.Num(), 0));
		SET_DWORD_STAT(STAT_ActiveSounds, ActiveSounds.Num());
		SET_DWORD_STAT(STAT_AudioVirtualLoops, VirtualLoops.Num());
		SET_DWORD_STAT(STAT_AudioMaxChannels, Channels);
		SET_DWORD_STAT(STAT_AudioMaxStoppingSources, NumStoppingSources);
	}

	// now let the platform perform anything it needs to handle
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_UpdateHardware);
		UpdateHardware();
	}

	// send any needed information back to the game thread
	SendUpdateResultsToGameThread(FirstActiveIndex);
}

void FAudioDevice::UpdateAudioVolumeEffects()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_UpdateAudioVolumeEffects);

	bool bHasVolumeSettings = false;
	FAudioVolumeSettings PlayerAudioVolumeSettings;
	bool bUsingDefaultReverb = true;

	FAudioVolumeSettings PreviousPlayerAudioVolumeSettings = CurrentAudioVolumeSettings;

	// Gets the current state of the interior settings
	for (FListener& Listener : Listeners)
	{
		FAudioVolumeSettings NewPlayerAudioVolumeSettings;
		GetAudioVolumeSettings(Listener.WorldID, Listener.Transform.GetLocation(), NewPlayerAudioVolumeSettings);

		Listener.ApplyInteriorSettings(NewPlayerAudioVolumeSettings.AudioVolumeID, NewPlayerAudioVolumeSettings.InteriorSettings);
		Listener.UpdateCurrentInteriorSettings();

		if (!bHasVolumeSettings || (NewPlayerAudioVolumeSettings.AudioVolumeID > 0 && (bUsingDefaultReverb || NewPlayerAudioVolumeSettings.Priority > PlayerAudioVolumeSettings.Priority)))
		{
			bHasVolumeSettings = true;
			PlayerAudioVolumeSettings = NewPlayerAudioVolumeSettings;

			if (NewPlayerAudioVolumeSettings.AudioVolumeID > 0)
			{
				if (NewPlayerAudioVolumeSettings.ReverbSettings.bApplyReverb)
				{
					bUsingDefaultReverb = false;
				}
				else if (GetDefaultAudioSettings(Listener.WorldID, NewPlayerAudioVolumeSettings.ReverbSettings, NewPlayerAudioVolumeSettings.InteriorSettings))
				{
					// Fall back to world reverb settings
					PlayerAudioVolumeSettings.ReverbSettings = NewPlayerAudioVolumeSettings.ReverbSettings;
				}
			}
		}
	}

	// Reset audio volume proxies to reset state back to before a proxy was updated
	ResetAudioVolumeProxyChangedState();

	// We need to update submix effects if we've entered a new volume or if the current volume settings have been updated
	bool bUpdateSubmixEffects = (PlayerAudioVolumeSettings.AudioVolumeID != CurrentAudioVolumeSettings.AudioVolumeID) ||
		(PlayerAudioVolumeSettings.AudioVolumeID == CurrentAudioVolumeSettings.AudioVolumeID && PlayerAudioVolumeSettings.bChanged);

	{
		// Lock for modifications to CurrentAudioVolumeSettings and contention with AddReferencedObjects(...)
		FScopeLock AudioThreadReferenceLock(&AudioThreadReferenceCritSec);

		CurrentAudioVolumeSettings = PlayerAudioVolumeSettings;

		if (Effects)
		{
			// Check if we should be using activated reverb
			if (bHasActivatedReverb && (HighestPriorityActivatedReverb_AudioThread.Priority > PlayerAudioVolumeSettings.Priority || bUsingDefaultReverb))
			{
				CurrentAudioVolumeSettings.ReverbSettings = HighestPriorityActivatedReverb_AudioThread.ReverbSettings;
			}

#if ENABLE_AUDIO_DEBUG
			// This is for debug visualization only - the audio debugger will show a reverb effect is active, if bApplyReverb is false for the world (default) reverb.
			// In order to not mislead people using au.debug.reverb, we manually set the reverb effect to none.  This doesn't impact non-debug enabled
			// builds, as a reverb effect w/ ApplyReverb false is equivalent to a null reverb effect that's been applied.
			else if (bUsingDefaultReverb && !CurrentAudioVolumeSettings.ReverbSettings.bApplyReverb)
			{
				CurrentAudioVolumeSettings.ReverbSettings.bApplyReverb = true;
				CurrentAudioVolumeSettings.ReverbSettings.ReverbEffect = nullptr;
			}
#endif // ENABLE_AUDIO_DEBUG

			// Update the master reverb if it has changed
			if (CurrentAudioVolumeSettings.bChanged || (CurrentAudioVolumeSettings.ReverbSettings != PreviousPlayerAudioVolumeSettings.ReverbSettings))
			{
				Effects->SetReverbSettings(CurrentAudioVolumeSettings.ReverbSettings);
			}

			// Update the audio effects - reverb, EQ etc
			Effects->Update();

			// If we any submix override settings apply those overrides to the indicated submixes
			if (bUpdateSubmixEffects)
			{
				// Clear out any previous submix effect chain overrides if the audio volume changed
				if (PreviousPlayerAudioVolumeSettings.SubmixOverrideSettings.Num() > 0)
				{
					for (FAudioVolumeSubmixOverrideSettings& OverrideSettings : PreviousPlayerAudioVolumeSettings.SubmixOverrideSettings)
					{
						ClearSubmixEffectChainOverride(OverrideSettings.Submix, OverrideSettings.CrossfadeTime);
					}
				}

				if (CurrentAudioVolumeSettings.SubmixOverrideSettings.Num() > 0)
				{
					for (FAudioVolumeSubmixOverrideSettings& OverrideSettings : CurrentAudioVolumeSettings.SubmixOverrideSettings)
					{
						if (OverrideSettings.Submix && OverrideSettings.SubmixEffectChain.Num() > 0)
						{

							FSoundEffectSubmixInitData InitData;
							InitData.SampleRate = GetSampleRate();

							TArray<FSoundEffectSubmixPtr> SubmixEffectPresetChainOverride;

							// Build the instances of the new submix preset chain override
							for (USoundEffectSubmixPreset* SubmixEffectPreset : OverrideSettings.SubmixEffectChain)
							{
								if (SubmixEffectPreset)
								{
									InitData.ParentPresetUniqueId = SubmixEffectPreset->GetUniqueID();

									TSoundEffectSubmixPtr SoundEffectSubmix = USoundEffectPreset::CreateInstance<FSoundEffectSubmixInitData, FSoundEffectSubmix>(InitData, *SubmixEffectPreset);
									SoundEffectSubmix->SetEnabled(true);
									SubmixEffectPresetChainOverride.Add(SoundEffectSubmix);
								}
							}

							SetSubmixEffectChainOverride(OverrideSettings.Submix, SubmixEffectPresetChainOverride, OverrideSettings.CrossfadeTime);
						}
					}
				}
			}
		}
	}
}

void FAudioDevice::UpdateAudioEngineSubsystems()
{
	ForEachSubsystem<UAudioEngineSubsystem>([](UAudioEngineSubsystem* Subsystem){
		if (Subsystem)
		{
			Subsystem->Update();
		}
	});
}

#if UE_AUDIO_PROFILERTRACE_ENABLED
void FAudioDevice::SendSoundStartedTraceMessage(const FActiveSound& InActiveSound) const
{
	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);

	if (bChannelEnabled)
	{
		const TObjectPtr<UObject> SoundObjectPtr = Audio::Trace::Util::GetSoundObjectPointer(InActiveSound);
		if (SoundObjectPtr == nullptr)
		{
			return;
		}

		const FString OwnerActorLabel = Audio::Trace::Util::GetOwnerActorLabel(InActiveSound);

		UE_TRACE_LOG(Audio, SoundStart, AudioChannel)
			<< SoundStart.DeviceId(InActiveSound.AudioDevice->DeviceID)
			<< SoundStart.Timestamp(FPlatformTime::Cycles64())
			<< SoundStart.PlayOrder(InActiveSound.PlayOrder)
			<< SoundStart.AssetPath(*SoundObjectPtr->GetPathName())
			<< SoundStart.EventLogName(*Audio::Trace::EventLog::ID::SoundStart)
			<< SoundStart.SoundClassName(*Audio::Trace::Util::GetSoundBaseAssetName(SoundObjectPtr->GetClass()))
			<< SoundStart.ActorLabel(*Audio::Trace::Util::GetOwnerActorLabel(InActiveSound))
			<< SoundStart.ActorIconName(*Audio::Trace::Util::GetOwnerActorIconName(InActiveSound).ToString())
			<< SoundStart.AudioComponentName(*Audio::Trace::Util::GetAudioComponentName(InActiveSound));
	}
}

void FAudioDevice::SendSoundIsAlivePingTraceMessage(const FActiveSound& InActiveSound) const
{
	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);

	if (bChannelEnabled)
	{
		const TObjectPtr<UObject> SoundObjectPtr = Audio::Trace::Util::GetSoundObjectPointer(InActiveSound);
		if (SoundObjectPtr == nullptr)
		{
			return;
		}

		const FString SoundBaseAssetName = Audio::Trace::Util::GetSoundBaseAssetName(SoundObjectPtr->GetClass());
		const FString OwnerActorLabel = Audio::Trace::Util::GetOwnerActorLabel(InActiveSound);

		UE_TRACE_LOG(Audio, SoundIsAlivePing, AudioChannel)
			<< SoundIsAlivePing.DeviceId(InActiveSound.AudioDevice->DeviceID)
			<< SoundIsAlivePing.Timestamp(FPlatformTime::Cycles64())
			<< SoundIsAlivePing.PlayOrder(InActiveSound.PlayOrder)
			<< SoundIsAlivePing.AssetPath(*SoundObjectPtr->GetPathName())
			<< SoundIsAlivePing.SoundClassName(*SoundBaseAssetName)
			<< SoundIsAlivePing.ActorLabel(*OwnerActorLabel)
			<< SoundIsAlivePing.AudioComponentName(*Audio::Trace::Util::GetAudioComponentName(InActiveSound));
	}
}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

FDelegateHandle FAudioDevice::AddPreRenderDelegate(const FOnAudioDevicePreRender::FDelegate& InDelegate)
{
	FScopeLock LockCallbacks(&RenderStateCallbackListCritSec);
	return OnAudioDevicePreRender.Add(InDelegate);
}

bool FAudioDevice::RemovePreRenderDelegate(const FDelegateHandle& InHandle)
{
	FScopeLock LockCallbacks(&RenderStateCallbackListCritSec);
	return OnAudioDevicePreRender.Remove(InHandle);
}

void FAudioDevice::NotifyAudioDevicePreRender(const FAudioDeviceRenderInfo& InInfo)
{
	FScopeLock LockCallbacks(&RenderStateCallbackListCritSec);
	OnAudioDevicePreRender.Broadcast(InInfo);
}

FDelegateHandle FAudioDevice::AddPostRenderDelegate(const FOnAudioDevicePostRender::FDelegate& InDelegate)
{
	FScopeLock LockCallbacks(&RenderStateCallbackListCritSec);
	return OnAudioDevicePostRender.Add(InDelegate);
}

bool FAudioDevice::RemovePostRenderDelegate(const FDelegateHandle& InHandle)
{
	FScopeLock LockCallbacks(&RenderStateCallbackListCritSec);
	return OnAudioDevicePostRender.Remove(InHandle);
}

void FAudioDevice::NotifyAudioDevicePostRender(const FAudioDeviceRenderInfo& InInfo)
{
	FScopeLock LockCallbacks(&RenderStateCallbackListCritSec);
	OnAudioDevicePostRender.Broadcast(InInfo);
}

void FAudioDevice::SendUpdateResultsToGameThread(const int32 FirstActiveIndex)
{
	DECLARE_CYCLE_STAT(TEXT("FGameThreadAudioTask.AudioSendResults"), STAT_AudioSendResults, STATGROUP_TaskGraphTasks);

	const Audio::FDeviceId AudioDeviceID = DeviceID;
	UReverbEffect* ReverbEffect = Effects ? Effects->GetCurrentReverbEffect() : nullptr;
	FAudioThread::RunCommandOnGameThread([AudioDeviceID, ReverbEffect]()
	{
		// At shutdown, GEngine may already be null
		if (GEngine)
		{
			if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
			{
				if (FAudioDeviceHandle AudioDevice = AudioDeviceManager->GetAudioDevice(AudioDeviceID))
				{
					AudioDevice->CurrentReverbEffect = ReverbEffect;
				}
			}
		}
	}, GET_STATID(STAT_AudioSendResults));

#if ENABLE_AUDIO_DEBUG
	Audio::FAudioDebugger::SendUpdateResultsToGameThread(*this, FirstActiveIndex);
#endif // ENABLE_AUDIO_DEBUG
}

void FAudioDevice::StopAllSounds(bool bShouldStopUISounds)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.StopAllSounds"), STAT_AudioStopAllSounds, STATGROUP_AudioThreadCommands);

		FAudioThread::RunCommandOnAudioThread([this, bShouldStopUISounds]()
		{
			StopAllSounds(bShouldStopUISounds);
		}, GET_STATID(STAT_AudioStopAllSounds));

		return;
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	Audio::Trace::EventLog::SendEvent(DeviceID, Audio::Trace::EventLog::ID::StopAllRequested);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	for (int32 SoundIndex=ActiveSounds.Num() - 1; SoundIndex >= 0; --SoundIndex)
	{
		FActiveSound* ActiveSound = ActiveSounds[SoundIndex];

		if (bShouldStopUISounds)
		{
			AddSoundToStop(ActiveSound);
		}
		// If we're allowing UI sounds to continue then first filter on the active sounds state
		else if (!ActiveSound->bIsUISound)
		{
			// Then iterate across the wave instances.  If any of the wave instances is not a UI sound
			// then we will stop the entire active sound because it makes less sense to leave it half
			// executing
			for (auto WaveInstanceIt(ActiveSound->WaveInstances.CreateConstIterator()); WaveInstanceIt; ++WaveInstanceIt)
			{
				FWaveInstance* WaveInstance = WaveInstanceIt.Value();
				if (WaveInstance && !WaveInstance->bIsUISound)
				{
					AddSoundToStop(ActiveSound);
					break;
				}
			}
		}
	}

	for (AudioDeviceUtils::FVirtualLoopPair& Pair : VirtualLoops)
	{
		AddSoundToStop(Pair.Key);
	}

	// Immediately process stopping sounds
	ProcessingPendingActiveSoundStops();
}

void FAudioDevice::InitializePluginListeners(UWorld* World)
{
	check(IsInGameThread());
	check(!bPluginListenersInitialized);

	AUDIO_SPATIALIZATION_PLUGIN_LLM_SCOPE

	for (TAudioPluginListenerPtr PluginListener : PluginListeners)
	{
		PluginListener->OnListenerInitialize(this, World);
	}
}

void FAudioDevice::NotifyPluginListenersWorldChanged(UWorld* World)
{
	check(IsInGameThread());

	AUDIO_SPATIALIZATION_PLUGIN_LLM_SCOPE

	for (TAudioPluginListenerPtr PluginListener : PluginListeners)
	{
		PluginListener->OnWorldChanged(this, World);
	}
}

void FAudioDevice::AddNewActiveSound(const FActiveSound& ActiveSound, const TArray<FAudioParameter>* InDefaultParams, const FString* EventLogID)
{
	TArray<FAudioParameter> Params;
	if (InDefaultParams)
	{
		Params.Append(*InDefaultParams);
	}
	//Previously the copy was done in AddNewActiveSoundInternal when creating the Lamba function. 
	TSharedRef<FActiveSound> NewUniqueActiveSound = MakeShared<FActiveSound>(ActiveSound);

	AddNewActiveSound(NewUniqueActiveSound, MoveTemp(Params), EventLogID);
}

void FAudioDevice::AddNewActiveSound(TSharedRef<FActiveSound> ActiveSound, TArray<FAudioParameter>&& InDefaultParams, const FString* EventLogID)
{
#if UE_AUDIO_PROFILERTRACE_ENABLED
	if (EventLogID)
	{
		Audio::Trace::EventLog::SendActiveSoundEvent(ActiveSound.Get(), *EventLogID);
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	if (USoundBase* Sound = ActiveSound.Get().GetSound())
	{
		Sound->InitResources();
	}
	AddNewActiveSoundInternal(ActiveSound, MoveTemp(InDefaultParams));

}

void FAudioDevice::AddNewActiveSound(const FActiveSound& NewActiveSound, TArray<FAudioParameter>&& InDefaultParams)
{
	//Previously the copy was done in AddNewActiveSoundInternal when creating the Lamba function. 
	TSharedRef<FActiveSound> NewUniqueActiveSound = MakeShared<FActiveSound>(NewActiveSound);

	AddNewActiveSound(NewUniqueActiveSound, MoveTemp(InDefaultParams));
}

void FAudioDevice::AddNewActiveSoundInternal(TSharedRef<FActiveSound> NewActiveSound, TArray<FAudioParameter>&& InDefaultParams, FAudioVirtualLoop* InVirtualLoopToRetrigger)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AddNewActiveSound"), STAT_AudioAddNewActiveSound, STATGROUP_AudioThreadCommands);

		checkf(nullptr == InVirtualLoopToRetrigger, TEXT("Virtual loops can only be retrigger from the audio thread"));

		FAudioThread::RunCommandOnAudioThread([AudioDevice = this, NewActiveSound = MoveTemp(NewActiveSound), DefaultParams = MoveTemp(InDefaultParams)]() mutable
			{
				AudioDevice->AddNewActiveSoundInternal(NewActiveSound, MoveTemp(DefaultParams));
			}, GET_STATID(STAT_AudioAddNewActiveSound));

		return;
	}

	AddNewActiveSoundInternal(NewActiveSound.Get(), MoveTemp(InDefaultParams), InVirtualLoopToRetrigger);

}

void FAudioDevice::AddNewActiveSoundInternal(const FActiveSound& InNewActiveSound, TArray<FAudioParameter>&& InDefaultParams, FAudioVirtualLoop* InVirtualLoopToRetrigger)
{
	LLM_SCOPE(ELLMTag::AudioMisc);
	
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AddNewActiveSound"), STAT_AudioAddNewActiveSound, STATGROUP_AudioThreadCommands);

		checkf(nullptr == InVirtualLoopToRetrigger, TEXT("Virtual loops can only be retrigger from the audio thread"));

		FAudioThread::RunCommandOnAudioThread([AudioDevice = this, InNewActiveSound, DefaultParams = MoveTemp(InDefaultParams)]() mutable
		{
			AudioDevice->AddNewActiveSoundInternal(InNewActiveSound, MoveTemp(DefaultParams));
		}, GET_STATID(STAT_AudioAddNewActiveSound));

		return;
	}

	USoundBase* Sound = InNewActiveSound.GetSound();
	// A non-playable sound could be a sound cue with nothing hooked to output.
	if (Sound == nullptr || !Sound->IsPlayable())
	{
		ReportSoundFailedToStart(InNewActiveSound.AudioComponentID, InVirtualLoopToRetrigger);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		Audio::Trace::EventLog::SendActiveSoundEvent(InNewActiveSound, Audio::Trace::EventLog::ID::PlayFailedNotPlayable);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		return;
	}

	if (Sound->GetDuration() <= FMath::Max(0.0f, SoundDistanceOptimizationLengthCVar))
	{
		// TODO: Determine if this check has already been completed at AudioComponent level and skip if so. Also,
		// unify code paths determining if sound is audible.
		if (!SoundIsAudible(InNewActiveSound))
		{
			UE_LOGF(LogAudio, Log, "New ActiveSound not created for out of range Sound %ls", *InNewActiveSound.Sound->GetName());

			ReportSoundFailedToStart(InNewActiveSound.AudioComponentID, InVirtualLoopToRetrigger);

#if UE_AUDIO_PROFILERTRACE_ENABLED
			Audio::Trace::EventLog::SendActiveSoundEvent(InNewActiveSound, Audio::Trace::EventLog::ID::PlayFailedOutOfRange);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			return;
		}
	}

	// Cull one-shot active sounds if we've reached our max limit of one shot active sounds before we attempt to evaluate concurrency
	// Check for debug sound name
#if !UE_BUILD_SHIPPING
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		FString DebugSound;
		if (AudioDeviceManager->GetDebugger().GetAudioDebugSound(DebugSound))
		{
			// Reject the new sound if it doesn't have the debug sound name substring
			FString SoundName;
			InNewActiveSound.Sound->GetName(SoundName);
			if (!SoundName.Contains(DebugSound))
			{
				ReportSoundFailedToStart(InNewActiveSound.AudioComponentID, InVirtualLoopToRetrigger);

#if UE_AUDIO_PROFILERTRACE_ENABLED
				Audio::Trace::EventLog::SendActiveSoundEvent(InNewActiveSound, Audio::Trace::EventLog::ID::PlayFailedDebugFiltered);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				return;
			}
		}
	}
#endif // !UE_BUILD_SHIPPING

	// Determine if sound is loop and eligible for virtualize prior to creating "live" active sound in next Concurrency check step
	if (!InVirtualLoopToRetrigger)
	{
		const bool bDoRangeCheck = true;
		FAudioVirtualLoop VirtualLoop;
		if (FAudioVirtualLoop::Virtualize(InNewActiveSound, *this, bDoRangeCheck, VirtualLoop))
		{
			UE_LOGF(LogAudio, Verbose, "New ActiveSound %ls Virtualizing: Failed to pass initial audible range check", *Sound->GetName());
			constexpr bool bIsVirtualLoopRealizing = false;
			InitSoundParams(VirtualLoop.GetActiveSound(), MoveTemp(InDefaultParams), bIsVirtualLoopRealizing);
			AddVirtualLoop(VirtualLoop);
			return;
		}
	}

	// Evaluate concurrency. This will create an ActiveSound ptr which is a copy of InNewActiveSound if the sound can play.
	FActiveSound* ActiveSound = nullptr;

	{
		SCOPE_CYCLE_COUNTER(STAT_AudioEvaluateConcurrency);

		// Try to create a new active sound. This returns nullptr if too many sounds are playing with this sound's concurrency setting
		ActiveSound = ConcurrencyManager.CreateNewActiveSound(InNewActiveSound, InVirtualLoopToRetrigger != nullptr);
	}

	// Didn't pass concurrency, and not an attempt to revive from virtualization, so see if candidate for virtualization
	if (!ActiveSound)
	{
		if (!InVirtualLoopToRetrigger)
		{
			const bool bDoRangeCheck = false;
			FAudioVirtualLoop VirtualLoop;
			if (FAudioVirtualLoop::Virtualize(InNewActiveSound, *this, bDoRangeCheck, VirtualLoop))
			{
				UE_LOGF(LogAudioConcurrency, Verbose, "New ActiveSound %ls Virtualizing: Failed to pass concurrency", *Sound->GetName());
				constexpr bool bIsVirtualLoopRealizing = false;
				InitSoundParams(VirtualLoop.GetActiveSound(), MoveTemp(InDefaultParams), bIsVirtualLoopRealizing);
				AddVirtualLoop(VirtualLoop);
			}
			else
			{
				ReportSoundFailedToStart(InNewActiveSound.GetAudioComponentID(), InVirtualLoopToRetrigger);

#if UE_AUDIO_PROFILERTRACE_ENABLED
				Audio::Trace::EventLog::SendActiveSoundEvent(InNewActiveSound, Audio::Trace::EventLog::ID::PlayFailedConcurrency);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
			}
		}
		return;
	}

	check(ActiveSound->Sound == Sound);

	if (GIsEditor)
	{
		// If the sound played on an editor preview world, treat it as a preview sound (unpausable and ignoring the realtime volume slider)
		if (const UWorld* World = InNewActiveSound.GetWorld())
		{
			ActiveSound->bIsPreviewSound |= (World->WorldType == EWorldType::EditorPreview);
		}
	}

	int32* PlayCount = Sound->CurrentPlayCount.Find(DeviceID);
	if (!PlayCount)
	{
		PlayCount = &Sound->CurrentPlayCount.Add(DeviceID);
	}
	(*PlayCount)++;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_LOGF(LogAudio, VeryVerbose, "New ActiveSound %ls Comp: %ls Loc: %ls", *Sound->GetName(), *InNewActiveSound.GetAudioComponentName(), *InNewActiveSound.Transform.GetTranslation().ToString());
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	// Cull one-shot active sounds if we've reached our max limit of one shot active sounds before we attempt to evaluate concurrency
	if (ActiveSound->IsOneShot())
	{
		OneShotCount++;
	}

	// Set the active sound to be playing audio so it gets parsed at least once.
	ActiveSound->bIsPlayingAudio = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!ensureMsgf(ActiveSound->Sound->GetFName() != NAME_None, TEXT("AddNewActiveSound with DESTROYED sound %s. AudioComponent=%s. IsValid=%d. BeginDestroy=%d"),
		*ActiveSound->Sound->GetPathName(),
		*ActiveSound->GetAudioComponentName(),
		(int32)IsValid(ActiveSound->Sound),
		(int32)ActiveSound->Sound->HasAnyFlags(RF_BeginDestroyed)))
	{
		static FName InvalidSoundName("DESTROYED_Sound");
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	// Loop has been successfully created, so add to stop before adding 'live' ActiveSound.
	// Mark to not report playback complete on destruction as responsibility therein has been
	// passed to newly created ActiveSound added below.  Add as stopping sound prior to adding
	// new sound to ensure AudioComponentIDToActiveSoundMap is registered with the correct ActiveSound.
	if (InVirtualLoopToRetrigger)
	{
		FActiveSound& VirtualActiveSound = InVirtualLoopToRetrigger->GetActiveSound();
		AddSoundToStop(&VirtualActiveSound);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		Audio::Trace::EventLog::SendActiveSoundEvent(VirtualActiveSound, Audio::Trace::EventLog::ID::SoundRealized);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		// Clear must be called after AddSoundToStop to ensure AudioComponent is properly removed from AudioComponentIDToActiveSoundMap
		VirtualActiveSound.ClearAudioComponent();

		// Attempt to seek the ActiveSound based on virtualization time
		if (ActiveSound->GetVirtualizationMode() == EVirtualizationMode::SeekRestart)
		{
			ActiveSound->SeekRestartTimes = InVirtualLoopToRetrigger->GetEffectivePlaybackTimes();
		}
	}

	InitSoundParams(*ActiveSound, MoveTemp(InDefaultParams), InVirtualLoopToRetrigger != nullptr /*bIsVirtualLoopRealzing*/);
	ActiveSounds.Add(ActiveSound);
	if (ActiveSound->GetAudioComponentID() > 0)
	{
		TArray<FActiveSound*>& ActiveSoundArray = AudioComponentIDToActiveSoundMap.FindOrAdd(ActiveSound->GetAudioComponentID());
		ActiveSoundArray.AddUnique(ActiveSound);
	}

	NotifySubsystemsActiveSoundCreated(*ActiveSound);

#if UE_AUDIO_PROFILERTRACE_ENABLED
	SendSoundStartedTraceMessage(*ActiveSound);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
}

void FAudioDevice::InitSoundParams(FActiveSound& InOutActiveSound, TArray<FAudioParameter> InDefaultParams, bool bInIsVirtualLoopRealizing) const
{
	check(IsInAudioThread());

	// Retriggering a virtualized ActiveSound which already have a transmitter
	// should not be given a new transmitter.
	if (!InOutActiveSound.InstanceTransmitter.IsValid())
	{
		const USoundBase* Sound = InOutActiveSound.GetSound();
		if (ensure(Sound))
		{
			Audio::FParameterTransmitterInitParams TransmitterInitParams
			{
				Audio::GetTransmitterID(InOutActiveSound.GetAudioComponentID(), 0, InOutActiveSound.GetPlayOrder()),
				GetSampleRate(),
				MoveTemp(InDefaultParams),
				DeviceID
			};

			InOutActiveSound.InstanceTransmitter = Sound->CreateParameterTransmitter(MoveTemp(TransmitterInitParams));
		}
	}
	else if (bInIsVirtualLoopRealizing)
	{
		// The default parameters should be empty when realizing a virtualized loop. Prior
		// parameters are already set on the instance transmitter. 
		InOutActiveSound.InstanceTransmitter->OnRealizeVirtualizedActiveSound(MoveTemp(InDefaultParams));
	}
	else
	{
		InOutActiveSound.InstanceTransmitter->SetParameters(MoveTemp(InDefaultParams));
	}
}

void FAudioDevice::ReportSoundFailedToStart(const uint64 AudioComponentID, FAudioVirtualLoop* VirtualLoop)
{
	check(IsInAudioThread());

	if (VirtualLoop)
	{
		FActiveSound& VirtualActiveSound = VirtualLoop->GetActiveSound();
		AddSoundToStop(&VirtualActiveSound);
	}
	else
	{
		const bool bFailedToStart = true;
		UAudioComponent::PlaybackCompleted(AudioComponentID, bFailedToStart);
	}
}

void FAudioDevice::RetriggerVirtualLoop(FAudioVirtualLoop& VirtualLoopToRetrigger)
{
	check(IsInAudioThread());

	AddNewActiveSoundInternal(VirtualLoopToRetrigger.GetActiveSound(), { }, &VirtualLoopToRetrigger);
}

void FAudioDevice::AddEnvelopeFollowerDelegate(USoundSubmix* InSubmix, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP)
{
	UE_LOGF(LogAudio, Error, "Envelope following submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
}

void FAudioDevice::RemoveEnvelopeFollowerDelegate(USoundSubmix* InSubmix, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP)
{
	UE_LOGF(LogAudio, Error, "Envelope following submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
}

void FAudioDevice::StartSpectrumAnalysis(USoundSubmix* InSubmix, const FSoundSpectrumAnalyzerSettings& InSettings)
{
	UE_LOGF(LogAudio, Error, "Spectrum analysis of submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
}

void FAudioDevice::StopSpectrumAnalysis(USoundSubmix* InSubmix)
{
	UE_LOGF(LogAudio, Error, "Spectrum analysis of submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
}

void FAudioDevice::GetMagnitudesForFrequencies(USoundSubmix* InSubmix, const TArray<float>& InFrequencies, TArray<float>& OutMagnitudes)
{
	UE_LOGF(LogAudio, Error, "Spectrum analysis of submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
}

void FAudioDevice::GetPhasesForFrequencies(USoundSubmix* InSubmix, const TArray<float>& InFrequencies, TArray<float>& OutPhases)
{
	UE_LOGF(LogAudio, Error, "Spectrum analysis of submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
}

void FAudioDevice::AddSpectralAnalysisDelegate(USoundSubmix* InSubmix, const FSoundSpectrumAnalyzerDelegateSettings& InDelegateSettings, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP)
{
	UE_LOGF(LogAudio, Error, "Spectrum analysis of submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
}

void FAudioDevice::RemoveSpectralAnalysisDelegate(USoundSubmix* InSubmix, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP)
{
	UE_LOGF(LogAudio, Error, "Spectrum analysis of submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
}

void FAudioDevice::AddVirtualLoop(const FAudioVirtualLoop& InVirtualLoop)
{
	FAudioVirtualLoop VirtualLoop = InVirtualLoop;

	FActiveSound& ActiveSound = VirtualLoop.GetActiveSound();
	check(!VirtualLoops.Contains(&ActiveSound));

	if (Audio::IParameterTransmitter* ParameterTransmitter = ActiveSound.GetTransmitter())
	{
		ParameterTransmitter->OnVirtualizeActiveSound();
	}

	// If associated with an AudioComponent, add the virtualizing ActiveSound pointer to the VirtualLoop system, 
	// and ensure it is in the AudioComponentIDToActiveSoundMap so updates from the AudioComponent are still tracked.
	const int64 ComponentID = ActiveSound.GetAudioComponentID();
	if (ComponentID > 0)
	{
		if (TArray<FActiveSound*>* ExistingSounds = AudioComponentIDToActiveSoundMap.Find(ComponentID))
		{
			// Only components playing a single looped sound at a time can virtualize.
			for (int32 i = ExistingSounds->Num() - 1; i >= 0; --i)
			{
				const FActiveSound* ExistingSound = (*ExistingSounds)[i];
				if (ensure(ExistingSound))
				{
					UE_LOGF(LogAudio, Warning, "Attempting to add Sound '%ls' to ComponentID when map already contains ID for Sound '%ls'. "
					"Associated AudioComponent can only play a single sound instance at one time. This may indicate a leak of FActiveSounds.",
						ActiveSound.Sound ? *ActiveSound.Sound->GetName() : TEXT("N/A"),
						ExistingSound->Sound ? *ExistingSound->Sound->GetName() : TEXT("N/A")
					);
					ExistingSounds->RemoveAtSwap(i, EAllowShrinking::No);
				}
			}
			ExistingSounds->AddUnique(&ActiveSound);
		}
		else
		{
			TArray<FActiveSound*>& NewActiveSoundArray = AudioComponentIDToActiveSoundMap.Add(ComponentID);
			NewActiveSoundArray.Add(&ActiveSound);
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (ActiveSound.Sound)
	{
		const FVector Location = ActiveSound.Transform.GetLocation();
		UE_LOGF(LogAudio, Verbose, "Adding virtual looping sound '%ls' at location %ls.", *ActiveSound.Sound->GetName(), *Location.ToCompactString());
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	VirtualLoops.Add(&ActiveSound, MoveTemp(VirtualLoop));

#if UE_AUDIO_PROFILERTRACE_ENABLED
	Audio::Trace::EventLog::SendActiveSoundEvent(ActiveSound, Audio::Trace::EventLog::ID::SoundVirtualized);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	NotifySubsystemsVirtualizedSoundCreated(ActiveSound);
}

bool FAudioDevice::RemoveVirtualLoop(FActiveSound& InActiveSound)
{
	check(IsInAudioThread());

	if (FAudioVirtualLoop* VirtualLoop = VirtualLoops.Find(&InActiveSound))
	{
		check(InActiveSound.bIsStopping);

		const uint64 ComponentID = InActiveSound.GetAudioComponentID();
		UAudioComponent::PlaybackCompleted(ComponentID, false);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (InActiveSound.Sound)
		{
			const FVector Location = InActiveSound.Transform.GetLocation();
			UE_LOGF(LogAudio, Verbose, "Removing virtual looping sound '%ls' with play order %d at location %ls.", *InActiveSound.Sound->GetName(), InActiveSound.GetPlayOrder(), *Location.ToCompactString());
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled)
		{
			UE_TRACE_LOG(Audio, VirtualLoopStopOrRealize, AudioChannel)
				<< VirtualLoopStopOrRealize.DeviceId(static_cast<uint32>(DeviceID))
				<< VirtualLoopStopOrRealize.Timestamp(FPlatformTime::Cycles64())
				<< VirtualLoopStopOrRealize.PlayOrder(InActiveSound.GetPlayOrder());
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		VirtualLoops.Remove(&InActiveSound);
		return true;
	}

	return false;
}

void FAudioDevice::ProcessingPendingActiveSoundStops(bool bForceDelete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_PendingActiveSoundStops);

	// Process the PendingSoundsToDelete. These may have
	// had their deletion deferred due to an async operation
	for (int32 i = PendingSoundsToDelete.Num() - 1; i >= 0; --i)
	{
		FActiveSound* ActiveSound = PendingSoundsToDelete[i];
		if (ActiveSound)
		{
			uint32 NumSourcesStopped = 0;

			bool bDeleteActiveSound = false;
			if (bForceDelete)
			{
				bDeleteActiveSound = true;
				// If we're in the process of stopping, but now we're force-deleting, make sure we finish the sound stopping
				if (ActiveSound->IsStopping())
				{
					// Make sure this sound finishes stopping if we're forcing all sounds to stop due to a flush, etc.
					bool bIsNowStopped = ActiveSound->UpdateStoppingSources(CurrentTick, true);
					check(bIsNowStopped);
				}
			}
			else if (ActiveSound->IsStopping())
			{
				// Update the stopping state. This will return true if we're ok to delete the active sound
				bDeleteActiveSound = ActiveSound->UpdateStoppingSources(CurrentTick, false);

				// If we are now deleting the active sound, then this is no longer stopping, so decrement the counter
				if (bDeleteActiveSound)
				{
					// It's possible we still may not be able to delete this sound if the active sound as a pending async task
					bDeleteActiveSound = ActiveSound->CanDelete();
				}
			}
			else if (ActiveSound->CanDelete())
			{
				bDeleteActiveSound = true;
			}

			if (bDeleteActiveSound)
			{
				if (ActiveSound->bIsPreviewSound && bModulationInterfaceEnabled && ModulationInterface.IsValid())
				{
					ModulationInterface->OnAuditionEnd();
				}
				ActiveSound->bAsyncOcclusionPending = false;
				PendingSoundsToDelete.RemoveAtSwap(i, EAllowShrinking::No);

				if (Audio::IParameterTransmitter* Transmitter = ActiveSound->GetTransmitter())
				{
					Transmitter->OnDeleteActiveSound();
				}
				ActiveSound->ClearTransmitter();

				NotifySubsystemsActiveSoundDeleting(*ActiveSound);
				delete ActiveSound;
			}
		}
	}

	while (PendingSoundsToStop.Num() > 0)
	{
		// Need to make a copy since stopping sounds can add to this list of sounds to stop when the audio thread isn't running (e.g. editor).
		TSet<FActiveSound*> PendingSoundsToStopCopy = PendingSoundsToStop;

		// Stop any pending active sounds that need to be stopped
		for (FActiveSound* ActiveSound : PendingSoundsToStopCopy)
		{
			check(ActiveSound);
			bool bDeleteActiveSound = false;
			bool bWasVirtualized = false;

			// If the request was to stop an ActiveSound that
			// is set to re-trigger but is not playing, remove
			// and continue
			if (RemoveVirtualLoop(*ActiveSound))
			{
				bDeleteActiveSound = true;
				bWasVirtualized = true;
			}
			else
			{
				ActiveSound->MarkPendingDestroy(bForceDelete);

				USoundBase* Sound = ActiveSound->GetSound();

				// If the active sound is a one shot, decrement the one shot counter
				if (Sound && !Sound->IsLooping())
				{
					OneShotCount--;
				}

				const bool bIsStopping = ActiveSound->IsStopping();

				// If we can delete the active sound now, then delete it
				if (bForceDelete || (ActiveSound->CanDelete() && !bIsStopping))
				{
					ActiveSound->bAsyncOcclusionPending = false;

					bDeleteActiveSound = true;
				}
				else
				{
					// There was an async operation pending or we are stopping (not stopped) so we need to defer deleting this sound
					PendingSoundsToDelete.AddUnique(ActiveSound);
				}
			}

			if (bDeleteActiveSound)
			{
				if (Audio::IParameterTransmitter* Transmitter = ActiveSound->GetTransmitter())
				{
					Transmitter->OnDeleteActiveSound();
				}
				ActiveSound->ClearTransmitter();

				if (bWasVirtualized)
				{
					NotifySubsystemsVirtualizedSoundDeleting(*ActiveSound);
				}
				else
				{
					NotifySubsystemsActiveSoundDeleting(*ActiveSound);
				}

				// Remove from the list of pending sounds to stop.
				// MUST be done last to avoid risk of double delete.
				PendingSoundsToStop.Remove(ActiveSound);

				delete ActiveSound;
			}
			else
			{
				// Remove from the list of pending sounds to stop
				PendingSoundsToStop.Remove(ActiveSound);
			}	
		}
	}
}

void FAudioDevice::AddSoundToStop(FActiveSound* SoundToStop)
{
	check(IsInAudioThread());
	check(SoundToStop);

	bool bAlreadyPending = false;
	PendingSoundsToStop.Add(SoundToStop, &bAlreadyPending);
	if (!bAlreadyPending)
	{
		const bool bIsVirtual = VirtualLoops.Contains(SoundToStop);
		if (bIsVirtual)
		{
			FAudioThread::RunCommandOnGameThread([AudioComponentID = SoundToStop->GetAudioComponentID()]()
			{
				if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
				{
					AudioComponent->SetIsVirtualized(false);
				}
			});
		}
		UnlinkActiveSoundFromComponent(*SoundToStop);

		if (bIsVirtual)
		{
			SoundToStop->bIsStopping = true;
		}
		else
		{
			ConcurrencyManager.RemoveActiveSound(*SoundToStop);
		}
	}
}

bool FAudioDevice::IsPendingStop(FActiveSound* ActiveSound)
{
	check(IsInAudioThread());
	check(ActiveSound);

	return PendingSoundsToStop.Contains(ActiveSound) || PendingSoundsToDelete.Contains(ActiveSound);
}

void FAudioDevice::StopActiveSound(const uint64 AudioComponentID, const FString* EventLogID)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.StopActiveSound"), STAT_AudioStopActiveSound, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, AudioComponentID, EventLogID]()
		{
			AudioDevice->StopActiveSound(AudioComponentID, EventLogID);
		}, GET_STATID(STAT_AudioStopActiveSound));

		return;
	}

	const Audio::FDeviceId AudioDeviceID = DeviceID;

	SendCommandToActiveSounds(AudioComponentID, [AudioDeviceID, EventLogID](FActiveSound& ActiveSound)
	{
		FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
			
		if (AudioDeviceManager)
		{
			FAudioDeviceHandle AudioDevice = AudioDeviceManager->GetAudioDevice(AudioDeviceID);
			if (AudioDevice.IsValid())
			{
#if UE_AUDIO_PROFILERTRACE_ENABLED
				if (EventLogID)
				{
					Audio::Trace::EventLog::SendActiveSoundEvent(ActiveSound, *EventLogID);
				}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				AudioDevice->AddSoundToStop(&ActiveSound);
			}
		}
	});
}

void FAudioDevice::StopActiveSound(FActiveSound* ActiveSound, const FString* EventLogID)
{
	check(IsInAudioThread());

#if UE_AUDIO_PROFILERTRACE_ENABLED
	if (ActiveSound && EventLogID)
	{
		Audio::Trace::EventLog::SendActiveSoundEvent(*ActiveSound, *EventLogID);
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	AddSoundToStop(ActiveSound);
}

void FAudioDevice::PauseActiveSound(const uint64 AudioComponentID, const bool bInIsPaused)
{
	check(IsInAudioThread());
	
	const Audio::FDeviceId AudioDeviceID = DeviceID;

	SendCommandToActiveSounds(AudioComponentID, [bInIsPaused, AudioDeviceID](FActiveSound& ActiveSound)
	{
		ActiveSound.bIsPaused = bInIsPaused;

#if UE_AUDIO_PROFILERTRACE_ENABLED
		if (GEngine)
		{
			FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();

			if (AudioDeviceManager)
			{
				FAudioDeviceHandle AudioDevice = AudioDeviceManager->GetAudioDevice(AudioDeviceID);
				if (AudioDevice.IsValid())
				{
					Audio::Trace::EventLog::SendActiveSoundEvent(ActiveSound, bInIsPaused ? Audio::Trace::EventLog::ID::PauseSoundRequested : Audio::Trace::EventLog::ID::ResumeSoundRequested);
				}
			}
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	});
}

void FAudioDevice::NotifyActiveSoundOcclusionTraceDone(FActiveSound* InActiveSound, bool bIsOccluded)
{
	// Find the active sound in these lists and only set these flags if they are in any of them
	if (ActiveSounds.Contains(InActiveSound) || PendingSoundsToStop.Contains(InActiveSound) || PendingSoundsToDelete.Contains(InActiveSound))
	{
		InActiveSound->bIsOccluded = bIsOccluded;
		InActiveSound->bAsyncOcclusionPending = false;
	}
}

FActiveSound* FAudioDevice::FindActiveSound(const uint64 AudioComponentID)
{
	ensure(IsInAudioThread());

	if (TArray<FActiveSound*>* ActiveSoundsInComponent = AudioComponentIDToActiveSoundMap.Find(AudioComponentID))
	{
		// return the first active sound corresponding to this audio component, if it exists, or a nullptr if not
		if (!ActiveSoundsInComponent->IsEmpty())
		{
			return (*ActiveSoundsInComponent)[0];
		}
	}

	return nullptr;
}

void FAudioDevice::SendCommandToActiveSounds(uint64 InAudioComponentID, TUniqueFunction<void(FActiveSound&)> InFunc, const TStatId InStatId)
{
	if (!IsInAudioThread())
	{
		FAudioThread::RunCommandOnAudioThread([this, InAudioComponentID, Func = MoveTemp(InFunc)]() mutable
		{
			SendCommandToActiveSounds(InAudioComponentID, MoveTemp(Func));
		}, InStatId);
		return;
	}

	// Must cache active sounds as the AudioComponentIDToActiveSoundMap can potentially be modified by the command function provided
	// (ex. when stopping a sound or array of sounds).
	const TArray<FActiveSound*> CachedActiveSounds = AudioComponentIDToActiveSoundMap.FindRef(InAudioComponentID);
	for (FActiveSound* ActiveSound : CachedActiveSounds)
	{
		// This should never be null
		if (ensure(ActiveSound))
		{
			InFunc(*ActiveSound);
		}
	}
}

bool FAudioDevice::CanHaveMultipleActiveSounds(uint64 AudioComponentID) const
{
	ensure(IsInAudioThread());

	if (const bool* bCanHaveMultipleActiveSounds = AudioComponentIDToCanHaveMultipleActiveSoundsMap.Find(AudioComponentID))
	{
		return *bCanHaveMultipleActiveSounds;
	}

	return false;
}

void FAudioDevice::SetCanHaveMultipleActiveSounds(uint64 InAudioComponentID, bool InCanHaveMultipleActiveSounds)
{
	if (!IsInAudioThread())
	{
		FAudioThread::RunCommandOnAudioThread([this, ComponentID = InAudioComponentID, bNewValue = InCanHaveMultipleActiveSounds]()
		{
			SetCanHaveMultipleActiveSounds(ComponentID, bNewValue);
		});

		return;
	}

	bool& bCanHaveMultipleActiveSounds = AudioComponentIDToCanHaveMultipleActiveSoundsMap.FindOrAdd(InAudioComponentID, InCanHaveMultipleActiveSounds);

	// This must be or'ed with existing value as disabling multiple active sounds while playing can potentially cause ActiveSound
	// instances to get lost while virtualizing.
	bCanHaveMultipleActiveSounds |= InCanHaveMultipleActiveSounds;
}

void FAudioDevice::RemoveActiveSound(FActiveSound* ActiveSound)
{
	check(IsInAudioThread());

	// Perform the notification if not sound not set to re-trigger
	const uint64 ComponentID = ActiveSound->GetAudioComponentID();
	UAudioComponent::PlaybackCompleted(ComponentID, false);

#if UE_AUDIO_PROFILERTRACE_ENABLED
	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
	if (bChannelEnabled)
	{
		const TObjectPtr<UObject> SoundObjectPtr = Audio::Trace::Util::GetSoundObjectPointer(*ActiveSound);
		if (SoundObjectPtr)
		{
			UE_TRACE_LOG(Audio, SoundStop, AudioChannel)
				<< SoundStop.DeviceId(ActiveSound->AudioDevice->DeviceID)
				<< SoundStop.Timestamp(FPlatformTime::Cycles64())
				<< SoundStop.PlayOrder(ActiveSound->PlayOrder)
				<< SoundStop.AssetPath(*SoundObjectPtr->GetPathName())
				<< SoundStop.EventLogName(*Audio::Trace::EventLog::ID::SoundStop)
				<< SoundStop.SoundClassName(*Audio::Trace::Util::GetSoundBaseAssetName(SoundObjectPtr->GetClass()))
				<< SoundStop.ActorLabel(*Audio::Trace::Util::GetOwnerActorLabel(*ActiveSound))
				<< SoundStop.ActorIconName(*Audio::Trace::Util::GetOwnerActorIconName(*ActiveSound).ToString())
				<< SoundStop.AudioComponentName(*Audio::Trace::Util::GetAudioComponentName(*ActiveSound));
		}
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	const int32 NumRemoved = ActiveSounds.RemoveSwap(ActiveSound);
	if (!ensureMsgf(NumRemoved > 0, TEXT("Attempting to remove an already removed ActiveSound '%s'"), ActiveSound->Sound ? *ActiveSound->Sound->GetName() : TEXT("N/A")))
	{
		return;
	}

	check(NumRemoved == 1);
}

bool FAudioDevice::LocationIsAudible(const FVector& Location, const float MaxDistance) const
{
	if (MaxDistance >= MaxWorldDistanceCVar)
	{
		return true;
	}

	const bool bInAudioThread = IsInAudioThread();
	const bool bInGameThread = IsInGameThread();

	check(bInAudioThread || bInGameThread);

	const int32 ListenerCount = bInAudioThread ? Listeners.Num() : ListenerProxies.Num();
	for (int32 i = 0; i < ListenerCount; ++i)
	{
		if (LocationIsAudible(Location, i, MaxDistance))
		{
			return true;
		}
	}

	return false;
}

bool FAudioDevice::LocationIsAudible(const FVector& Location, int32 ListenerIndex, float MaxDistance) const
{
	if (MaxDistance >= MaxWorldDistanceCVar)
	{
		return true;
	}
	
	FVector ListenerTranslation;
	const bool bAllowOverride = true;
	if (ListenerIndex == INDEX_NONE || !GetListenerPosition(ListenerIndex, ListenerTranslation, bAllowOverride))
	{
		return false;
	}
	
	const float MaxDistanceSquared = MaxDistance * MaxDistance;
	return (ListenerTranslation - Location).SizeSquared() < MaxDistanceSquared;
}

float FAudioDevice::GetDistanceToNearestListener(const FVector& Location) const
{
	float DistSquared = 0.0f;
	if (GetDistanceSquaredToNearestListener(Location, DistSquared))
	{
		return FMath::Sqrt(DistSquared);
	}

	return MaxWorldDistanceCVar;
}

bool FAudioDevice::GetDistanceSquaredToListener(const FVector& Location, int32 ListenerIndex, float& OutSqDistance) const
{
	OutSqDistance = TNumericLimits<float>::Max();
	const int32 ListenerCount = IsInAudioThread() ? Listeners.Num() : ListenerProxies.Num();

	if (ListenerIndex >= ListenerCount)
	{
		return false;
	}

	FVector ListenerTranslation;
	const bool bAllowOverride = true;
	if (!GetListenerPosition(ListenerIndex, ListenerTranslation, bAllowOverride))
	{
		return false;
	}

	OutSqDistance = (ListenerTranslation - Location).SizeSquared();
	return true;
}

bool FAudioDevice::GetDistanceSquaredToNearestListener(const FVector& Location, float& OutSqDistance) const
{
	OutSqDistance = TNumericLimits<float>::Max();
	const bool bInAudioThread = IsInAudioThread();
	const bool bInGameThread = IsInGameThread();

	check(bInAudioThread || bInGameThread);

	float DistSquared;
	const bool bAllowAttenuationOverrides = true;
	if (FindClosestListenerIndex(Location, DistSquared, bAllowAttenuationOverrides) == INDEX_NONE)
	{
		OutSqDistance = MaxWorldDistanceCVar;
		return false;
	}

	OutSqDistance = DistSquared;
	return true;
}

float FAudioDevice::GetMaxWorldDistance()
{
	return MaxWorldDistanceCVar;
}

bool FAudioDevice::GetListenerPosition(int32 ListenerIndex, FVector& OutPosition, bool bAllowOverride) const
{
	OutPosition = FVector::ZeroVector;
	if (ListenerIndex == INDEX_NONE)
	{
		return false;
	}

	if (IsInAudioThread())
	{
		checkf(ListenerIndex < Listeners.Num(), TEXT("Listener Index %u out of range of available Listeners!"), ListenerIndex);
		const FListener& Listener = Listeners[ListenerIndex];
		OutPosition = Listener.GetPosition(bAllowOverride);
		return true;
	}
	else // IsInGameThread()
	{
		checkf(ListenerIndex < ListenerProxies.Num(), TEXT("Listener Index %u out of range of available Listeners!"), ListenerIndex);
		const FListenerProxy& Proxy = ListenerProxies[ListenerIndex];
		OutPosition = Proxy.GetPosition(bAllowOverride);
		return true;
	}
}

bool FAudioDevice::GetListenerTransform(int32 ListenerIndex, FTransform& OutTransform) const
{
	OutTransform.SetIdentity();
	if (ListenerIndex == INDEX_NONE)
	{
		return false;
	}

	if (IsInAudioThread())
	{
		if (ListenerIndex < Listeners.Num())
		{
			OutTransform = Listeners[ListenerIndex].Transform;
			return true;
		}
	}
	else // IsInGameThread()
	{
		if (ListenerIndex < ListenerProxies.Num())
		{
			OutTransform = ListenerProxies[ListenerIndex].Transform;
			return true;
		}
	}

	return false;
}

bool FAudioDevice::GetListenerWorldID(int32 ListenerIndex, uint32& OutWorldID) const
{
	OutWorldID = INDEX_NONE;
	if (ListenerIndex == INDEX_NONE)
	{
		return false;
	}

	if (IsInAudioThread())
	{
		if (ListenerIndex < Listeners.Num())
		{
			OutWorldID = Listeners[ListenerIndex].WorldID;
			return true;
		}
	}
	else // IsInGameThread()
	{
		if (ListenerIndex < ListenerProxies.Num())
		{
			OutWorldID = ListenerProxies[ListenerIndex].WorldID;
			return true;
		}
	}

	return false;
}

void FAudioDevice::GetMaxDistanceAndFocusFactor(USoundBase* Sound, const UWorld* World, const FVector& Location, const FSoundAttenuationSettings* AttenuationSettingsToApply, float& OutMaxDistance, float& OutFocusFactor)
{
	check(IsInGameThread());
	check(Sound);

	const bool bHasAttenuationSettings = ShouldUseAttenuation(World) && AttenuationSettingsToApply;

	OutFocusFactor = 1.0f;

	if (bHasAttenuationSettings)
	{
		FTransform SoundTransform;
		SoundTransform.SetTranslation(Location);

		OutMaxDistance = AttenuationSettingsToApply->GetMaxDimension();
		if (AttenuationSettingsToApply->AttenuationShape == EAttenuationShape::Box)
		{
			static const float Sqrt2 = 1.4142135f;
			OutMaxDistance *= Sqrt2;
		}

		if (AttenuationSettingsToApply->bSpatialize && AttenuationSettingsToApply->bEnableListenerFocus)
		{
			const int32 ClosestListenerIndex = FindClosestListenerIndex(SoundTransform);
			if (ClosestListenerIndex == INDEX_NONE)
			{
				UE_LOGF(LogAudio, Warning, "Invalid ClosestListenerIndex. Sound max distance and focus factor calculation failed.");
				return;
			}

			// Now scale the max distance based on the focus settings in the attenuation settings
			FAttenuationListenerData ListenerData = FAttenuationListenerData::Create(*this, ClosestListenerIndex, SoundTransform, *AttenuationSettingsToApply);

			float Azimuth = 0.0f;
			float AbsoluteAzimuth = 0.0f;
			GetAzimuth(ListenerData, Azimuth, AbsoluteAzimuth);
			OutFocusFactor = GetFocusFactor(Azimuth, *AttenuationSettingsToApply);
		}
	}
	else
	{
		// No need to scale the distance by focus factor since we're not using any attenuation settings
		OutMaxDistance = Sound->GetMaxDistance();
	}
}

bool FAudioDevice::SoundIsAudible(USoundBase* Sound, const UWorld* World, const FVector& Location, const FSoundAttenuationSettings* AttenuationSettingsToApply, float MaxDistance, float FocusFactor) const
{
	check(IsInGameThread());

	const bool bHasAttenuationSettings = ShouldUseAttenuation(World) && AttenuationSettingsToApply;
	float DistanceScale = 1.0f;
	if (bHasAttenuationSettings)
	{
		// If we are not using distance-based attenuation, this sound will be audible regardless of distance.
		if (!AttenuationSettingsToApply->bAttenuate)
		{
			return true;
		}

		DistanceScale = AttenuationSettingsToApply->GetFocusDistanceScale(GetGlobalFocusSettings(), FocusFactor);
	}

	DistanceScale = FMath::Max(DistanceScale, 0.0001f);
	return LocationIsAudible(Location, MaxDistance / DistanceScale);
}

bool FAudioDevice::SoundIsAudible(const FActiveSound& NewActiveSound)
{
	check(NewActiveSound.Sound);

	// If we have an attenuation node, we can't know until we evaluate
	// the sound cue if it's audio output going to be audible via a
	// distance check. TODO: Check if this is still the case.
	if (NewActiveSound.Sound->HasAttenuationNode())
	{
		return true;
	}

	if (PlayWhenSilentEnabled() && (NewActiveSound.Sound->SupportsSubtitles() || (NewActiveSound.bHandleSubtitles && NewActiveSound.bHasExternalSubtitles)))
	{
		return true;
	}

	if (NewActiveSound.Sound->IsPlayWhenSilent())
	{
		return true;
	}

	// TODO: bAllowSpatialization is used in other audibility checks but not here.
	const FSoundAttenuationSettings& Attenuation = NewActiveSound.AttenuationSettings;
	const bool bHasFocusScaling = Attenuation.FocusDistanceScale != 1.0f || Attenuation.NonFocusDistanceScale != 1.0f;
	if (!NewActiveSound.bHasAttenuationSettings ||
		(NewActiveSound.bHasAttenuationSettings && (!Attenuation.bAttenuate || bHasFocusScaling)))
	{
		return true;
	}

	// TODO: Check if this is necessary. GetMaxDistanceAndFocusFactor should've solved this and would make this
	// flavor of SoundIsAudible more accurate.
	const FGlobalFocusSettings& FocusSettings = GetGlobalFocusSettings();
	if (FocusSettings.FocusDistanceScale != 1.0f || FocusSettings.NonFocusDistanceScale != 1.0f)
	{
		return true;
	}

	const float ApparentMaxDistance = NewActiveSound.MaxDistance * NewActiveSound.FocusData.DistanceScale;
	if (LocationIsAudible(NewActiveSound.Transform.GetLocation(), ApparentMaxDistance))
	{
		return true;
	}

	return false;
}

int32 FAudioDevice::FindClosestListenerIndex(const FTransform& SoundTransform, const TArray<FListener>& InListeners)
{
	check(IsInAudioThread());
	int32 ClosestListenerIndex = 0;
	const bool bAllowAttenuationOverride = true;
	if (InListeners.Num() > 0)
	{
		float ClosestDistSq = FVector::DistSquared(SoundTransform.GetTranslation(), InListeners[0].GetPosition(bAllowAttenuationOverride));

		for (int32 i = 1; i < InListeners.Num(); i++)
		{
			const float DistSq = FVector::DistSquared(SoundTransform.GetTranslation(), InListeners[i].GetPosition(bAllowAttenuationOverride));
			if (DistSq < ClosestDistSq)
			{
				ClosestListenerIndex = i;
				ClosestDistSq = DistSq;
			}
		}
	}

	return ClosestListenerIndex;
}

int32 FAudioDevice::FindClosestListenerIndex(const FTransform& SoundTransform) const
{
	float UnusedDistSq;
	const bool bAllowOverrides = true;
	return FindClosestListenerIndex(SoundTransform.GetTranslation(), UnusedDistSq, bAllowOverrides);
}

int32 FAudioDevice::FindClosestListenerIndex(const FVector& Position, float& OutDistanceSq, bool bAllowAttenuationOverrides) const
{
	int32 ClosestListenerIndex = 0;
	OutDistanceSq = 0.f;
	FVector ListenerPosition;

	if (!GetListenerPosition(0, ListenerPosition, bAllowAttenuationOverrides))
	{
		return INDEX_NONE;
	}

	OutDistanceSq = FVector::DistSquared(Position, ListenerPosition);

	const int32 ListenerCount = IsInAudioThread() ? Listeners.Num() : ListenerProxies.Num();
	for (int32 i = 1; i < ListenerCount; ++i)
	{
		if (!GetListenerPosition(i, ListenerPosition, bAllowAttenuationOverrides))
		{
			continue;
		}

		const float DistSq = FVector::DistSquared(Position, ListenerPosition);
		if (DistSq < OutDistanceSq)
		{
			OutDistanceSq = DistSq;
			ClosestListenerIndex = i;
		}
	}

	return ClosestListenerIndex;
}

void FAudioDevice::UnlinkActiveSoundFromComponent(const FActiveSound& InActiveSound)
{
	const uint64 AudioComponentID = InActiveSound.GetAudioComponentID();
	if (AudioComponentID > 0)
	{
		if (TArray<FActiveSound*>* ActiveSoundsInComponent = AudioComponentIDToActiveSoundMap.Find(AudioComponentID))
		{
			for (int32 i = ActiveSoundsInComponent->Num() - 1; i >= 0; --i)
			{
				FActiveSound* ActiveSound = (*ActiveSoundsInComponent)[i];
				if (ensure(ActiveSound))
				{
					if (ActiveSound->GetInstanceID() == InActiveSound.GetInstanceID())
					{
						ActiveSoundsInComponent->RemoveAtSwap(i, EAllowShrinking::No);
						break;
					}
				}
			}
			
			if (ActiveSoundsInComponent->IsEmpty())
			{
				AudioComponentIDToActiveSoundMap.Remove(AudioComponentID);
				AudioComponentIDToCanHaveMultipleActiveSoundsMap.Remove(AudioComponentID);
			}
		}

	}
}

void FAudioDevice::GetAzimuth(const FAttenuationListenerData& ListenerData, float& OutAzimuth, float& OutAbsoluteAzimuth) const
{
	const FVector& ListenerForwardDir = ListenerData.ListenerTransform.GetUnitAxis(EAxis::X);

	const float SoundToListenerForwardDotProduct = FVector::DotProduct(ListenerForwardDir, ListenerData.ListenerToSoundDir);
	const float SoundListenerAngleRadians = FMath::Acos(SoundToListenerForwardDotProduct);

	// Normal azimuth only goes to 180 (0 is in front, 180 is behind).
	OutAzimuth = FMath::RadiansToDegrees(SoundListenerAngleRadians);

	const FVector& ListenerRightDir = ListenerData.ListenerTransform.GetUnitAxis(EAxis::Y);
	const float SoundToListenerRightDotProduct = FVector::DotProduct(ListenerRightDir, ListenerData.ListenerToSoundDir);

	FVector AbsAzimuthVector2D = FVector(SoundToListenerForwardDotProduct, SoundToListenerRightDotProduct, 0.0f);
	AbsAzimuthVector2D.Normalize();

	OutAbsoluteAzimuth = FMath::IsNearlyZero(AbsAzimuthVector2D.X) ? UE_HALF_PI : FMath::Atan(AbsAzimuthVector2D.Y / AbsAzimuthVector2D.X);
	OutAbsoluteAzimuth = FMath::RadiansToDegrees(OutAbsoluteAzimuth);
	OutAbsoluteAzimuth = FMath::Abs(OutAbsoluteAzimuth);

	if (AbsAzimuthVector2D.X > 0.0f && AbsAzimuthVector2D.Y < 0.0f)
	{
		OutAbsoluteAzimuth = 360.0f - OutAbsoluteAzimuth;
	}
	else if (AbsAzimuthVector2D.X < 0.0f && AbsAzimuthVector2D.Y < 0.0f)
	{
		OutAbsoluteAzimuth += 180.0f;
	}
	else if (AbsAzimuthVector2D.X < 0.0f && AbsAzimuthVector2D.Y > 0.0f)
	{
		OutAbsoluteAzimuth = 180.0f - OutAbsoluteAzimuth;
	}
}

float FAudioDevice::GetFocusFactor(const float Azimuth, const FSoundAttenuationSettings& AttenuationSettings) const
{
	// 0.0f means we are in focus, 1.0f means we are out of focus
	float FocusFactor = 0.0f;

	const float FocusAzimuth = FMath::Clamp(GlobalFocusSettings.FocusAzimuthScale * AttenuationSettings.FocusAzimuth, 0.0f, 180.0f);
	const float NonFocusAzimuth = FMath::Clamp(GlobalFocusSettings.NonFocusAzimuthScale * AttenuationSettings.NonFocusAzimuth, 0.0f, 180.0f);

	if (FocusAzimuth != NonFocusAzimuth)
	{
		FocusFactor = (Azimuth - FocusAzimuth) / (NonFocusAzimuth - FocusAzimuth);
		FocusFactor = FMath::Clamp(FocusFactor, 0.0f, 1.0f);
	}
	else if (Azimuth >= FocusAzimuth)
	{
		FocusFactor = 1.0f;
	}

	return FocusFactor;
}

FAudioDevice::FCreateComponentParams::FCreateComponentParams()
	: World(nullptr)
	, Actor(nullptr)
{
	AudioDevice = (GEngine ? GEngine->GetMainAudioDeviceRaw() : nullptr);
	CommonInit();
}

FAudioDevice::FCreateComponentParams::FCreateComponentParams(UWorld* InWorld, AActor* InActor)
	: World(InWorld)
{
	if (InActor)
	{
		check(InActor->GetWorld() == InWorld);
		Actor = InActor;
	}
	else
	{
		Actor = (World ? World->GetWorldSettings() : nullptr);
	}

	AudioDevice = (World ? World->GetAudioDeviceRaw() : nullptr);
	
	// If the world doesn't own an audio device, fall back to the main audio device.
	if (!AudioDevice)
	{
		AudioDevice = (GEngine ? GEngine->GetMainAudioDeviceRaw() : nullptr);
	}

	CommonInit();
}

FAudioDevice::FCreateComponentParams::FCreateComponentParams(AActor* InActor)
	: Actor(InActor)
{
	World = (Actor ? Actor->GetWorld() : nullptr);
	AudioDevice = (World ? World->GetAudioDeviceRaw() : nullptr);

	// If the world doesn't own an audio device, fall back to the main audio device.
	if (!AudioDevice)
	{
		AudioDevice = (GEngine ? GEngine->GetMainAudioDeviceRaw() : nullptr);
	}

	CommonInit();
}

FAudioDevice::FCreateComponentParams::FCreateComponentParams(FAudioDevice* InAudioDevice)
	: World(nullptr)
	, Actor(nullptr)
	, AudioDevice(InAudioDevice)
{
	CommonInit();
}

void FAudioDevice::FCreateComponentParams::CommonInit()
{
	bAutoDestroy = true;
	bPlay = false;
	bStopWhenOwnerDestroyed = true;
	bLocationSet = false;
	AttenuationSettings = nullptr;
	ConcurrencySet.Reset();
	Location = FVector::ZeroVector;
}

void FAudioDevice::FCreateComponentParams::SetLocation(const FVector InLocation)
{
	if (World)
	{
		bLocationSet = true;
		Location = InLocation;
	}
	else
	{
		UE_LOGF(LogAudio, Warning, "AudioComponents created without a World cannot have a location.");
	}
}

bool FAudioDevice::FCreateComponentParams::ShouldUseAttenuation() const
{
	if (AudioDevice)
	{
		return AudioDevice->ShouldUseAttenuation(World);
	}

	return true;
}

UAudioComponent* FAudioDevice::CreateComponent(USoundBase* Sound, const FCreateComponentParams& Params)
{
	check(IsInGameThread());

	UAudioComponent* AudioComponent = nullptr;

	if (Sound && Params.AudioDevice && GEngine && GEngine->UseSound())
	{
		// Avoid creating component if we're trying to play a sound on an already destroyed actor.
		if (Params.Actor == nullptr || IsValid(Params.Actor))
		{
			// Listener position could change before long sounds finish
			const FSoundAttenuationSettings* AttenuationSettingsToApply = (Params.AttenuationSettings ? &Params.AttenuationSettings->Attenuation : Sound->GetAttenuationSettingsToApply());

			bool bIsAudible = true;
			// If a sound is a long duration, the position might change before sound finishes so assume it's audible
			if (Params.bLocationSet && Sound->GetDuration() <= FMath::Max(0.0f, SoundDistanceOptimizationLengthCVar))
			{
				float MaxDistance = 0.0f;
				float FocusFactor = 0.0f;
				Params.AudioDevice->GetMaxDistanceAndFocusFactor(Sound, Params.World, Params.Location, AttenuationSettingsToApply, MaxDistance, FocusFactor);
				bIsAudible = Params.AudioDevice->SoundIsAudible(Sound, Params.World, Params.Location, AttenuationSettingsToApply, MaxDistance, FocusFactor);
			}

			if (bIsAudible)
			{
				// Use actor as outer if we have one.
				if (Params.Actor)
				{
					AudioComponent = NewObject<UAudioComponent>(Params.Actor, (Params.AudioComponentClass != nullptr) ? (UClass*)Params.AudioComponentClass : UAudioComponent::StaticClass());
				}
				// Let engine pick the outer (transient package).
				else
				{
					AudioComponent = NewObject<UAudioComponent>(GetTransientPackage(), (Params.AudioComponentClass != nullptr) ? (UClass*)Params.AudioComponentClass : UAudioComponent::StaticClass());
				}

				check(AudioComponent);

				AudioComponent->Sound = Sound;
				AudioComponent->bAutoActivate = false;
				AudioComponent->bIsUISound = false;
				AudioComponent->bAutoDestroy = Params.bPlay && Params.bAutoDestroy;
				AudioComponent->bStopWhenOwnerDestroyed = Params.bStopWhenOwnerDestroyed;
#if WITH_EDITORONLY_DATA
				AudioComponent->bVisualizeComponent = false;
#endif
				AudioComponent->AttenuationSettings = Params.AttenuationSettings;
				AudioComponent->ConcurrencySet = Params.ConcurrencySet;

				if (Params.bLocationSet)
				{
					AudioComponent->SetWorldLocation(Params.Location);
				}

				// AudioComponent used in PlayEditorSound sets World to nullptr to avoid situations where the world becomes invalid
				// and the component is left with invalid pointer.
				if (Params.World)
				{
					AudioComponent->RegisterComponentWithWorld(Params.World);
				}
				else
				{
					AudioComponent->AudioDeviceID = Params.AudioDevice->DeviceID;
				}

				if (Params.bPlay)
				{
					AudioComponent->Play();
				}
			}
			else
			{
				// Don't create a sound component for short sounds that start out of range of any listener
				UE_LOGF(LogAudio, Log, "AudioComponent not created for out of range Sound %ls", *Sound->GetName());
			}
		}
	}

	return AudioComponent;
}

void FAudioDevice::PlaySoundAtLocation(USoundBase* Sound, UWorld* World, float VolumeMultiplier, float PitchMultiplier, float StartTime, const FVector& Location, const FRotator& Rotation, USoundAttenuation* AttenuationSettings, USoundConcurrency* Concurrency, TArray<FAudioParameter>&& Params, const AActor* OwningActor)
{
	check(IsInGameThread());

	if (!Sound || !World)
	{
		return;
	}

	// Not audible if the ticking level collection is not visible
	if (World && World->GetActiveLevelCollection() && !World->GetActiveLevelCollection()->IsVisible())
	{
		return;
	}

	const FSoundAttenuationSettings* AttenuationSettingsToApply = (AttenuationSettings ? &AttenuationSettings->Attenuation : Sound->GetAttenuationSettingsToApply());
	float MaxDistance = 0.0f;
	float FocusFactor = 1.0f;

	GetMaxDistanceAndFocusFactor(Sound, World, Location, AttenuationSettingsToApply, MaxDistance, FocusFactor);

	if (Sound->IsLooping() || Sound->IsPlayWhenSilent() || SoundIsAudible(Sound, World, Location, AttenuationSettingsToApply, MaxDistance, FocusFactor))
	{
		const bool bIsInGameWorld = World->IsGameWorld();
		TSharedRef<FActiveSound> NewSharedActiveSound = MakeShared<FActiveSound>();
		FActiveSound& NewActiveSound = NewSharedActiveSound.Get();
		NewActiveSound.SetWorld(World);
		NewActiveSound.SetSound(Sound);
		NewActiveSound.SetVolume(VolumeMultiplier);
		NewActiveSound.SetPitch(PitchMultiplier);
		NewActiveSound.RequestedStartTime = FMath::Max(0.0f, StartTime);
		NewActiveSound.bLocationDefined = true;
		NewActiveSound.Transform.SetTranslation(Location);
		NewActiveSound.Transform.SetRotation(FQuat(Rotation));
		NewActiveSound.bIsUISound = !bIsInGameWorld;
		NewActiveSound.SubtitlePriority = Sound->GetSubtitlePriority();

		NewActiveSound.bHasAttenuationSettings = (ShouldUseAttenuation(World) && AttenuationSettingsToApply);
		if (NewActiveSound.bHasAttenuationSettings)
		{
			const FGlobalFocusSettings& FocusSettings = GetGlobalFocusSettings();

			NewActiveSound.AttenuationSettings = *AttenuationSettingsToApply;
			NewActiveSound.FocusData.PriorityScale = AttenuationSettingsToApply->GetFocusPriorityScale(FocusSettings, FocusFactor);
			NewActiveSound.FocusData.DistanceScale = AttenuationSettingsToApply->GetFocusDistanceScale(FocusSettings, FocusFactor);
		}

		NewActiveSound.MaxDistance = MaxDistance;

		if (Concurrency)
		{
			NewActiveSound.ConcurrencySet.Add(Concurrency);
		}

		NewActiveSound.Priority = Sound->Priority;

		NewActiveSound.SetOwner(OwningActor);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const FString* EventLogID = &Audio::Trace::EventLog::ID::PlayRequestSoundAtLocation;
#else
		const FString* EventLogID = nullptr;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		AddNewActiveSound(MoveTemp(NewSharedActiveSound), MoveTemp(Params), EventLogID);
	}
	else
	{
		// Don't play a sound for short sounds that start out of range of any listener
		UE_LOGF(LogAudio, Log, "Sound not played for out of range Sound %ls", *Sound->GetName());
	}
}


void FAudioDevice::PlaySoundAtLocation(USoundBase* Sound, UWorld* World, float VolumeMultiplier, float PitchMultiplier, float StartTime, const FVector& Location, const FRotator& Rotation, USoundAttenuation* AttenuationSettings, USoundConcurrency* Concurrency, const TArray<FAudioParameter>* Params, const AActor* OwningActor)
{
	check(IsInGameThread());

	TArray<FAudioParameter> ParamsCopy;
	if (Params)
	{
		ParamsCopy.Append(*Params);
	}

	PlaySoundAtLocation(Sound, World, VolumeMultiplier, PitchMultiplier, StartTime, Location, Rotation, AttenuationSettings, Concurrency, MoveTemp(ParamsCopy), OwningActor);
}

void FAudioDevice::Flush(UWorld* WorldToFlush, bool bClearActivatedReverb)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.Flush"), STAT_AudioFlush, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, WorldToFlush]()
		{
			AudioDevice->Flush(WorldToFlush);
		}, GET_STATID(STAT_AudioFlush));

		FAudioCommandFence AudioFence;
		AudioFence.BeginFence();
		AudioFence.Wait();

		// Clear the GameThread cache of the listener
		ListenerProxies.Reset();
		ListenerProxies.AddDefaulted();

		return;
	}

	for (int32 i = PrecachingSoundWaves.Num() - 1; i >= 0; --i)
	{
		USoundWave* Wave = PrecachingSoundWaves[i];
		if (Wave->CleanupDecompressor(true))
		{
			PrecachingSoundWaves.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}

	// Do fadeout when flushing the audio device.
	if (WorldToFlush == nullptr)
	{
		FadeOut();
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	Audio::Trace::EventLog::SendEvent(DeviceID, Audio::Trace::EventLog::ID::FlushAudioDeviceRequested);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	// Stop all audio components attached to the scene
	bool bFoundIgnoredComponent = false;
	for (int32 Index = ActiveSounds.Num() - 1; Index >= 0; --Index)
	{
		FActiveSound* ActiveSound = ActiveSounds[Index];
		// if we are in the editor we want to always flush the ActiveSounds
		if (WorldToFlush && ActiveSound->bIgnoreForFlushing)
		{
			bFoundIgnoredComponent = true;
		}
		else
		{
			if (WorldToFlush == nullptr)
			{
				AddSoundToStop(ActiveSound);
			}
			else
			{
				UWorld* ActiveSoundWorld = ActiveSound->World.Get();
				if (ActiveSoundWorld == nullptr || ActiveSoundWorld == WorldToFlush)
				{
					AddSoundToStop(ActiveSound);
				}
			}
		}
	}

	// We use a copy as some operations may modify VirtualLoops
	{
		TMap<FActiveSound*, FAudioVirtualLoop> VirtualLoopsCopy = VirtualLoops;
		for (AudioDeviceUtils::FVirtualLoopPair& Pair : VirtualLoopsCopy)
		{
			AddSoundToStop(Pair.Key);
		}
	}

	// Immediately stop all pending active sounds
	ProcessingPendingActiveSoundStops(WorldToFlush == nullptr || WorldToFlush->bIsTearingDown);

	// Anytime we flush, make sure to clear all the listeners.  We'll get the right ones soon enough.
	Listeners.Reset();
	Listeners.Add(FListener(this));

	// Clear all the activated reverb effects
	if (bClearActivatedReverb)
	{
		ActivatedReverbs.Reset();
		bHasActivatedReverb = false;
	}

	if (WorldToFlush == nullptr)
	{
		// Make sure sounds are fully stopped.
		if (bFoundIgnoredComponent)
		{
			// We encountered an ignored component, so address the sounds individually.
			// There's no need to individually clear WaveInstanceSourceMap elements,
			// because FSoundSource::Stop(...) takes care of this.
			for (int32 SourceIndex = 0; SourceIndex < Sources.Num(); SourceIndex++)
			{
				const FWaveInstance* WaveInstance = Sources[SourceIndex]->GetWaveInstance();
				if (WaveInstance == nullptr || !WaveInstance->ActiveSound->bIgnoreForFlushing)
				{
					Sources[ SourceIndex ]->Stop();
				}
			}
		}
		else
		{
			// No components were ignored, so stop all sounds.
			for (int32 SourceIndex = 0; SourceIndex < Sources.Num(); SourceIndex++)
			{
				Sources[ SourceIndex ]->Stop();
			}

			WaveInstanceSourceMap.Reset();
		}
	}

	if (WorldToFlush == nullptr)
	{
		ReferencedSoundWaves.Reset();
	}

	// Make sure we update any hardware changes that need to happen after flushing
	UpdateHardware();

	// Make sure any in-flight audio rendering commands get executed.
	FlushAudioRenderingCommands();

	FlushExtended(WorldToFlush, bClearActivatedReverb);
}

void FAudioDevice::FlushExtended(UWorld* WorldToFlush, bool bClearActivatedReverb)
{
}

/**
 * Precaches the passed in sound node wave object.
 *
 * @param	SoundWave	Resource to be precached.
 */

void FAudioDevice::Precache(USoundWave* SoundWave, bool bSynchronous, bool bTrackMemory, bool bForceFullDecompression)
{
	using namespace AudioDeviceUtils;

	LLM_SCOPE(ELLMTag::AudioPrecache);

	if (SoundWave == nullptr)
	{
		return;
	}


	// We're already precaching this sound wave so no need to precache again
	if (SoundWave->DecompressionType != DTYPE_Setup && !bForceFullDecompression)
	{
		return;
	}

	if (bForceFullDecompression)
	{
		SoundWave->SetPrecacheState(ESoundWavePrecacheState::NotStarted);
	}

	if (!bSynchronous && SoundWave->GetPrecacheState() == ESoundWavePrecacheState::NotStarted)
	{
		if (!bForceFullDecompression && DisableAutomaticPrecacheCvar == 1)
		{
			// Don't schedule a precache for a normal async request because it is currently disabled
			return;
		}

		if (IsInGameThread())
		{
			// On the game thread, add this sound wave to the referenced sound wave nodes so that it doesn't get GC'd
			SoundWave->SetPrecacheState(ESoundWavePrecacheState::InProgress);
			ReferencedSoundWaves.AddUnique(SoundWave);
		}

		// Precache is called from USoundWave::PostLoad, from the game thread, and thus function needs to be called from the audio thread
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.Precache"), STAT_AudioPrecache, STATGROUP_AudioThreadCommands);

			FAudioDevice* AudioDevice = this;
			FAudioThread::RunCommandOnAudioThread([AudioDevice, SoundWave, bSynchronous, bTrackMemory, bForceFullDecompression]()
			{
				AudioDevice->Precache(SoundWave, bSynchronous, bTrackMemory, bForceFullDecompression);
			}, GET_STATID(STAT_AudioPrecache));

			return;
		}
	}

	// calculate the decompression type
	// @todo audio: maybe move this into SoundWave?
	if (SoundWave->NumChannels == 0)
	{
		// No channels - no way of knowing what to play back
		SoundWave->DecompressionType = DTYPE_Invalid;
	}
	else if (SoundWave->bProcedural)
	{
		// Procedurally created audio
		SoundWave->DecompressionType = DTYPE_Procedural;
	}
	else if (SoundWave->bIsSourceBus)
	{
		// Buses will initialize as procedural, but not actually become a procedural sound wave
		SoundWave->DecompressionType = DTYPE_Procedural;
	}
	else
	{
		check(HasCompressedAudioInfoClass(SoundWave));
		const FSoundGroup& SoundGroup = GetDefault<USoundGroups>()->GetSoundGroup(SoundWave->SoundGroup);

		if (SoundWave->Duration <= 0.0f)
		{
			UE_LOGF(LogAudio, Warning, "Sound Wave reported a duration of zero. This will likely result in incorrect decoding.");
		}
		
		static FName NAME_OGG("OGG");
		SoundWave->bDecompressedFromOgg = SoundWave->GetRuntimeFormat() == NAME_OGG;
		bool bDecompressToPCM = false;

		// handle audio decompression
		if (FPlatformProperties::SupportsAudioStreaming() && SoundWave->IsStreaming(nullptr))
		{
			SoundWave->DecompressionType = DTYPE_Streaming;
			SoundWave->bCanProcessAsync = false;
		}
		else if (!bForceFullDecompression) 
		{
			// Store as compressed data and decompress in realtime
			SoundWave->DecompressionType = DTYPE_RealTime;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			++PrecachedRealtime;
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		}
		else
		{
			// Fully expand loaded audio data into PCM
			bDecompressToPCM = true;

			// if the decompression type is already DTYPE_RealTime from a previous call
			// the task will early exit and we won't decompress
			SoundWave->DecompressionType = DTYPE_Setup;
		}

		// Grab the compressed audio data
		SoundWave->InitAudioResource(SoundWave->GetRuntimeFormat());

		if (SoundWave->AudioDecompressor == nullptr && (bDecompressToPCM || SoundWave->DecompressionType == DTYPE_RealTime))
		{
			// Create a worker to decompress the audio data
			if (bSynchronous)
			{
				// Create a worker to decompress the vorbis data
				FAsyncAudioDecompress TempDecompress(SoundWave, GetNumPrecacheFrames(), this);
				TempDecompress.StartSynchronousTask();
			}
			else
			{
				// This is the one case where precaching will not be done when this function exits
				checkf(SoundWave->GetPrecacheState() == ESoundWavePrecacheState::InProgress, TEXT("Bad PrecacheState %d on SoundWave %s"), static_cast<uint8>(SoundWave->GetPrecacheState()), *GetPathNameSafe(SoundWave));
				SoundWave->AudioDecompressor = new FAsyncAudioDecompress(SoundWave, GetNumPrecacheFrames(), this);
				SoundWave->AudioDecompressor->StartBackgroundTask();
				PrecachingSoundWaves.Add(SoundWave);
			}
		}
	}

	// If we don't have an audio decompressor task, then we're fully precached
	if (!SoundWave->AudioDecompressor)
	{
		SoundWave->SetPrecacheState(ESoundWavePrecacheState::Done);
	}

	if (bTrackMemory)
	{
		const int32 ResourceSize = SoundWave->GetResourceSizeBytes(EResourceSizeMode::Exclusive);
		SoundWave->TrackedMemoryUsage += ResourceSize;

		// If we aren't decompressing it above, then count the memory
		INC_DWORD_STAT_BY(STAT_AudioMemorySize, ResourceSize);
		INC_DWORD_STAT_BY(STAT_AudioMemory, ResourceSize);
	}
}

float FAudioDevice::GetCompressionDurationThreshold(const FSoundGroup &SoundGroup)
{
	// Check to see if the compression duration threshold is overridden via CVar:
	float CompressedDurationThreshold = 0.f; // related cvar has been removed
	
	// If not, check to see if there is an override for the compression duration on this platform in the project settings:
	if (CompressedDurationThreshold <= 0.0f)
	{
		CompressedDurationThreshold = FPlatformCompressionUtilities::GetCompressionDurationForCurrentPlatform();
	}

	// If there is neither a CVar override nor a runtime setting override, use the decompression threshold from the sound group directly:
	if (CompressedDurationThreshold < 0.0f)
	{
		CompressedDurationThreshold = SoundGroup.DecompressedDuration;
	}

	return CompressedDurationThreshold;
}

// (deprecated)
bool FAudioDevice::ShouldUseRealtimeDecompression(bool bForceFullDecompression, const FSoundGroup &SoundGroup, USoundWave* SoundWave, float CompressedDurationThreshold) const
{
	return !bForceFullDecompression;
}

// deprecated
void FAudioDevice::StopSourcesUsingBuffer(FSoundBuffer* SoundBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioDevice_StopSourcesUsingBuffer);

	check(IsInAudioThread());

	// if (SoundBuffer)
	// {
	// 	for (int32 SrcIndex = 0; SrcIndex < Sources.Num(); SrcIndex++)
	// 	{
	// 		FSoundSource* Src = Sources[SrcIndex];
	// 		if (Src && Src->Buffer == SoundBuffer) // deprecated: Src->Buffer no longer exists
	// 		{
	// 			// Make sure the buffer is no longer referenced by anything
	// 			Src->StopNow();
	// 			break;
	// 		}
	// 	}
	// }
}

void FAudioDevice::RegisterSoundClass(USoundClass* InSoundClass)
{
	if (InSoundClass)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.RegisterSoundClass"), STAT_AudioRegisterSoundClass, STATGROUP_AudioThreadCommands);

			FAudioDevice* AudioDevice = this;
			FAudioThread::RunCommandOnAudioThread([AudioDevice, InSoundClass]()
			{
				AudioDevice->RegisterSoundClass(InSoundClass);
			}, GET_STATID(STAT_AudioRegisterSoundClass));

			return;
		}

		// If the sound class wasn't already registered get it in to the system.
		if (!SoundClasses.Contains(InSoundClass))
		{
			SoundClasses.Add(InSoundClass, FSoundClassProperties());

			FSoundClassDynamicProperties NewDynamicProperties;
			NewDynamicProperties.AttenuationScaleParam.Set(InSoundClass->Properties.AttenuationDistanceScale, 0.0f);
			DynamicSoundClassProperties.Add(InSoundClass, NewDynamicProperties);
		}
	}
}

void FAudioDevice::UnregisterSoundClass(USoundClass* InSoundClass)
{
	if (InSoundClass)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.UnregisterSoundClass"), STAT_AudioUnregisterSoundClass, STATGROUP_AudioThreadCommands);

			FAudioDevice* AudioDevice = this;
			FAudioThread::RunCommandOnAudioThread([AudioDevice, InSoundClass]()
			{
				AudioDevice->UnregisterSoundClass(InSoundClass);
			}, GET_STATID(STAT_AudioUnregisterSoundClass));

			return;
		}

		SoundClasses.Remove(InSoundClass);
		DynamicSoundClassProperties.Remove(InSoundClass);
	}
}

USoundSubmix& FAudioDevice::GetMainSubmixObject() const
{
	UE_LOGF(LogAudio, Error, "Main submix accessor only supported with the audio mixer. Run with audio mixer enabled. Class Default Object returned.");
	USoundSubmix* CDO = USoundSubmix::StaticClass()->GetDefaultObject<USoundSubmix>();
	check(CDO);
	return *CDO;
}

void FAudioDevice::RegisterSubmixBufferListener(ISubmixBufferListener* InSubmixBufferListener, USoundSubmix* SoundSubmix)
{
	UE_LOGF(LogAudio, Error, "Submix buffer listener only works with the audio mixer. Please run with audio mixer enabled.");
}

void FAudioDevice::RegisterSubmixBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe>, USoundSubmix& SoundSubmix)
{
	UE_LOGF(LogAudio, Error, "Submix buffer listener only works with the audio mixer. Please run with audio mixer enabled.");
}

void FAudioDevice::UnregisterSubmixBufferListener(ISubmixBufferListener* InSubmixBufferListener, USoundSubmix* SoundSubmix)
{
	UE_LOGF(LogAudio, Error, "Submix buffer listener only works with the audio mixer. Please run with audio mixer enabled.");
}

void FAudioDevice::UnregisterSubmixBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe>, USoundSubmix& SoundSubmix)
{
	UE_LOGF(LogAudio, Error, "Submix buffer listener only works with the audio mixer. Please run with audio mixer enabled.");
}

Audio::FPatchOutputStrongPtr FAudioDevice::AddPatchForSubmix(uint32 InObjectId, float InPatchGain)
{
	UE_LOGF(LogAudio, Error, "Submix patching only works with the audio mixer. Please run with audio mixer enabled.");
	return nullptr;
}

Audio::FPatchInput FAudioDevice::AddPatchInputForAudioBus(uint32 InAudioBusId, int32 InFrames, int32 InChannels, float InGain)
{
	return Audio::FPatchInput();
}

Audio::FPatchOutputStrongPtr FAudioDevice::AddPatchOutputForAudioBus(uint32 InAudioBusId, int32 InFrames, int32 InChannels, float InGain)
{
	return Audio::FPatchOutputStrongPtr();
}

FSoundClassProperties* FAudioDevice::GetSoundClassCurrentProperties(USoundClass* InSoundClass)
{
	if (InSoundClass)
	{
		check(IsInAudioThread());

		FSoundClassProperties* Properties = SoundClasses.Find(InSoundClass);
		return Properties;
	}
	return nullptr;
}

FSoundClassDynamicProperties* FAudioDevice::GetSoundClassDynamicProperties(USoundClass* InSoundClass)
{
	if (InSoundClass)
	{
		check(IsInAudioThread());

		FSoundClassDynamicProperties* Properties = DynamicSoundClassProperties.Find(InSoundClass);
		return Properties;
	}
	return nullptr;
}

void FAudioDevice::StopSoundsUsingResource(USoundWave* SoundWave, TArray<UAudioComponent*>* StoppedComponents)
{
	if (StoppedComponents == nullptr && !IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.StopSoundsUsingResource"), STAT_AudioStopSoundsUsingResource, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, SoundWave]()
		{
			AudioDevice->StopSoundsUsingResource(SoundWave);
		}, GET_STATID(STAT_AudioStopSoundsUsingResource));

		return;
	}
	else if (StoppedComponents)
	{
		check(IsInGameThread());
		FAudioCommandFence AudioFence;
		AudioFence.BeginFence();
		AudioFence.Wait();
	}

	bool bStoppedSounds = false;

	for (int32 ActiveSoundIndex = ActiveSounds.Num() - 1; ActiveSoundIndex >= 0; --ActiveSoundIndex)
	{
		FActiveSound* ActiveSound = ActiveSounds[ActiveSoundIndex];
		for (const TPair<UPTRINT, FWaveInstance*>& WaveInstancePair : ActiveSound->WaveInstances)
		{
			// If anything the ActiveSound uses the wave then we stop the sound
			FWaveInstance* WaveInstance = WaveInstancePair.Value;
			if (WaveInstance && WaveInstance->WaveData == SoundWave)
			{
				if (StoppedComponents)
				{
					if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(ActiveSound->GetAudioComponentID()))
					{
						StoppedComponents->Add(AudioComponent);
					}
				}

#if UE_AUDIO_PROFILERTRACE_ENABLED
				Audio::Trace::EventLog::SendActiveSoundEvent(*ActiveSound, Audio::Trace::EventLog::ID::StopRequestSoundsUsingResource);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				AddSoundToStop(ActiveSound);
				bStoppedSounds = true;
				break;
			}
		}
	}

	// Immediately stop all pending active sounds
	ProcessingPendingActiveSoundStops();

	if (!GIsEditor && bStoppedSounds)
	{
		UE_LOGF(LogAudio, Verbose, "All Sounds using SoundWave '%ls' have been stopped", *SoundWave->GetName());
	}
}

bool FAudioDevice::LegacyReverbDisabled()
{
	return DisableLegacyReverb != 0;
}

void FAudioDevice::RegisterPluginListener(const TAudioPluginListenerPtr PluginListener)
{
	PluginListeners.AddUnique(PluginListener);
}

void FAudioDevice::UnregisterPluginListener(const TAudioPluginListenerPtr PluginListener)
{
	PluginListeners.RemoveSingle(PluginListener);
}

bool FAudioDevice::IsAudioDeviceMuted() const
{
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	if(DeviceManager)
	{
		// Check to see if the device manager has "bPlayAllPIEAudio" enabled
		const bool bIsPlayAllDeviceAudio = DeviceManager->IsPlayAllDeviceAudio();

		// Check if always playing NonRealtime devices, and this is a NonRealtime device
		const bool bIsAlwaysPlayNonRealtime = DeviceManager->IsAlwaysPlayNonRealtimeDeviceAudio() && IsNonRealtime();

		if (bIsPlayAllDeviceAudio || bIsAlwaysPlayNonRealtime)
		{
			return false;
		}

		// If we have one active device, ignore device muting
		if (DeviceManager->GetNumActiveAudioDevices() == 1)
		{
			return false;
		}
	}

	return bIsDeviceMuted;
}

void FAudioDevice::SetDeviceMuted(const bool bMuted)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetDeviceMuted"), STAT_SetDeviceMuted, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, bMuted]()
		{
			AudioDevice->SetDeviceMuted(bMuted);

		}, GET_STATID(STAT_SetDeviceMuted));

		return;
	}

	bIsDeviceMuted = bMuted;
}

FVector FAudioDevice::GetListenerTransformedDirection(const FVector& Position, float* OutDistance)
{
	float DistSquared;
	const bool bAllowAttenuationOverrides = true;

	int32 ClosestListenerIndex = FindClosestListenerIndex(Position, DistSquared, bAllowAttenuationOverrides);
	if (ClosestListenerIndex == INDEX_NONE || ClosestListenerIndex >= InverseListenerTransforms.Num())
	{
		UE_LOGF(LogAudioMixer, Display, "Invalid listener index (%d) when trying to find listener-transformed direction", ClosestListenerIndex);
		return FVector::ForwardVector;
	}

	check(IsInAudioThread());
	FVector UnnormalizedDirection = InverseListenerTransforms[ClosestListenerIndex].TransformPosition(Position);
	if (OutDistance)
	{
		*OutDistance = UnnormalizedDirection.Size();
	}
	return UnnormalizedDirection.GetSafeNormal();
}

float FAudioDevice::GetDeviceDeltaTime() const
{
	// Clamp the delta time to a reasonable max delta time.
	return FMath::Min(DeviceDeltaTime, 0.5f);
}

float FAudioDevice::GetGameDeltaTime() const
{
	float DeltaTime = FApp::GetDeltaTime();

	// Clamp the delta time to a reasonable max delta time.
	return FMath::Min(DeltaTime, 0.5f);
}

bool FAudioDevice::IsUsingListenerAttenuationOverride(int32 ListenerIndex) const
{
	const bool bInAudioThread = IsInAudioThread();
	const int32 ListenerCount = bInAudioThread ? Listeners.Num() : ListenerProxies.Num();
	if (ListenerIndex >= ListenerCount)
	{
		return false;
	}

	if (bInAudioThread)
	{
		return Listeners[ListenerIndex].bUseAttenuationOverride;
	}

	return ListenerProxies[ListenerIndex].bUseAttenuationOverride;
}

const FVector& FAudioDevice::GetListenerAttenuationOverride(int32 ListenerIndex) const
{
	const bool bInAudioThread = IsInAudioThread();
	const int32 ListenerCount = bInAudioThread ? Listeners.Num() : ListenerProxies.Num();
	check(ListenerIndex < ListenerCount);

	if (bInAudioThread)
	{
		return Listeners[ListenerIndex].AttenuationOverride;
	}

	return ListenerProxies[ListenerIndex].AttenuationOverride;
}

void FAudioDevice::UpdateVirtualLoops(bool bForceUpdate)
{
	using namespace AudioDeviceUtils;

	check(IsInAudioThread());

	if (FAudioVirtualLoop::IsEnabled())
	{
		TArray<FAudioVirtualLoop> VirtualLoopsToRetrigger;

		for (FVirtualLoopPair& Pair : VirtualLoops)
		{
			FAudioVirtualLoop& VirtualLoop = Pair.Value;
			FActiveSound& ActiveSound = VirtualLoop.GetActiveSound();

			// Don't update if stopping.
			if (ActiveSound.bIsStopping)
			{
				continue;
			}

			// If signaled to fade out and virtualized, add to pending stop list.
			if (ActiveSound.FadeOut != FActiveSound::EFadeOut::None)
			{
				AddSoundToStop(&ActiveSound);
				continue;
			}

			// If the loop is ready to realize, add to array to be re-triggered
			// outside of the loop to avoid map manipulation while iterating.
			if (VirtualLoop.Update(GetDeviceDeltaTime(), bForceUpdate))
			{
				VirtualLoopsToRetrigger.Add(VirtualLoop);
			}
		}

		for (FAudioVirtualLoop& RetriggerLoop : VirtualLoopsToRetrigger)
		{
			RetriggerVirtualLoop(RetriggerLoop);
		}
	}

	// if !FAudioVirtualLoop::IsEnabled(), attempt to realize/re-trigger
	// sounds and remove virtual loops.
	else
	{
		// Copies any straggling virtual loops to active sounds and mark them for stop
		for (FVirtualLoopPair& Pair : VirtualLoops)
		{
			FActiveSound* ActiveSound = Pair.Key;
			check(ActiveSound);

			UnlinkActiveSoundFromComponent(*ActiveSound);
			AddNewActiveSound(*ActiveSound);

			ActiveSound->ClearAudioComponent();
			AddSoundToStop(ActiveSound);
		}
	}
}

#if WITH_EDITOR
void FAudioDevice::OnBeginPIE(const bool bIsSimulating)
{
	for (TObjectIterator<USoundNode> It; It; ++It)
	{
		USoundNode* SoundNode = *It;
		SoundNode->OnBeginPIE(bIsSimulating);
	}

#if ENABLE_AUDIO_DEBUG
	Audio::FAudioDebugger::OnBeginPIE();
#endif // ENABLE_AUDIO_DEBUG
}

void FAudioDevice::OnEndPIE(const bool bIsSimulating)
{
	for (TObjectIterator<USoundNode> It; It; ++It)
	{
		USoundNode* SoundNode = *It;
		SoundNode->OnEndPIE(bIsSimulating);
	}

#if ENABLE_AUDIO_DEBUG
	Audio::FAudioDebugger::OnEndPIE();
#endif // ENABLE_AUDIO_DEBUG
}
#endif // WITH_EDITOR

bool FAudioDevice::CanUseVRAudioDevice()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		UEditorEngine* EdEngine = Cast<UEditorEngine>(GEngine);
		return EdEngine ? EdEngine->IsVRPreviewActive() : false;
	}
	else
#endif
	{
		return IStereoRendering::IsStartInVR();
	}
}

void FAudioDevice::SetTransientPrimaryVolume(const float InTransientPrimaryVolume)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetTransientPrimaryVolume"), STAT_SetTransientPrimaryVolume, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, InTransientPrimaryVolume]()
		{
			AudioDevice->SetTransientPrimaryVolume(InTransientPrimaryVolume);
		}, GET_STATID(STAT_SetTransientPrimaryVolume));

		return;
	}

	TransientPrimaryVolume = InTransientPrimaryVolume;
}

FSoundSource* FAudioDevice::GetSoundSource(FWaveInstance* WaveInstance) const
{
	check(IsInAudioThread());
	return WaveInstanceSourceMap.FindRef( WaveInstance );
}

const FGlobalFocusSettings& FAudioDevice::GetGlobalFocusSettings() const
{
	if (IsInAudioThread())
	{
		return GlobalFocusSettings;
	}

	check(IsInGameThread());
	return GlobalFocusSettings_GameThread;
}

void FAudioDevice::SetGlobalFocusSettings(const FGlobalFocusSettings& NewFocusSettings)
{
	check(IsInGameThread());

	GlobalFocusSettings_GameThread = NewFocusSettings;

	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetGlobalListenerFocusParameters"), STAT_AudioSetGlobalListenerFocusParameters, STATGROUP_TaskGraphTasks);
	FAudioThread::RunCommandOnAudioThread([this, NewFocusSettings]()
	{
		GlobalFocusSettings = NewFocusSettings;
	}, GET_STATID(STAT_AudioSetGlobalListenerFocusParameters));
}

void FAudioDevice::SetGlobalPitchModulation(float PitchModulation, float TimeSec)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetGlobalPitchModulation"), STAT_AudioSetGlobalPitchModulation, STATGROUP_TaskGraphTasks);

		FAudioThread::RunCommandOnAudioThread([this, PitchModulation, TimeSec]()
		{
			SetGlobalPitchModulation(PitchModulation, TimeSec);
		}, GET_STATID(STAT_AudioSetGlobalPitchModulation));

		return;
	}

	GlobalPitchScale.Set(PitchModulation, TimeSec);
}

void FAudioDevice::SetSoundClassDistanceScale(USoundClass* InSoundClass, float DistanceScale, float TimeSec)
{
	check(InSoundClass);

	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetSoundClassDistanceScale"), STAT_AudioSetSoundClassDistanceScale, STATGROUP_TaskGraphTasks);

		FAudioThread::RunCommandOnAudioThread([this, InSoundClass, DistanceScale, TimeSec]()
		{
			SetSoundClassDistanceScale(InSoundClass, DistanceScale, TimeSec);
		}, GET_STATID(STAT_AudioSetSoundClassDistanceScale));

		return;
	}

	if (FSoundClassDynamicProperties* DynamicProperties = DynamicSoundClassProperties.Find(InSoundClass))
	{
		DynamicProperties->AttenuationScaleParam.Set(DistanceScale, TimeSec);
	}
}

float FAudioDevice::ClampPitch(float InPitchScale) const
{
	return FMath::Clamp(InPitchScale, GlobalMinPitch, GlobalMaxPitch);
}

void FAudioDevice::SetPlatformAudioHeadroom(const float InPlatformHeadRoom)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetPlatformAudioHeadroom"), STAT_SetPlatformAudioHeadroom, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, InPlatformHeadRoom]()
		{
			AudioDevice->SetPlatformAudioHeadroom(InPlatformHeadRoom);
		}, GET_STATID(STAT_SetPlatformAudioHeadroom));

		return;
	}

	PlatformAudioHeadroom = InPlatformHeadRoom;
}

bool FAudioDevice::IsMainAudioDevice() const
{
	FAudioDeviceHandle MainAudioDevice = GEngine->GetMainAudioDevice();
	return (!MainAudioDevice || MainAudioDevice.GetAudioDevice() == this);
}

const TArray<FWaveInstance*>& FAudioDevice::GetActiveWaveInstances() const
{
	check(IsInAudioThread());
	return ActiveWaveInstances;
}

const TMap<FName, FActivatedReverb>& FAudioDevice::GetActiveReverb() const
{
	return ActivatedReverbs;
}

const TMap<FWaveInstance*, FSoundSource*>& FAudioDevice::GetWaveInstanceSourceMap() const
{
	return WaveInstanceSourceMap;
}

FName FAudioDevice::GetAudioStateProperty(const FName& PropertyName) const
{
	check(IsInAudioThread());
	if (const FName* Property = AudioStateProperties.Find(PropertyName))
	{
		return *Property;
	}
	
	return FName();
}

void FAudioDevice::SetAudioStateProperty(const FName& PropertyName, const FName& PropertyValue)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetAudioStateProperty"), STAT_SetAudioStateProperty, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, PropertyName, PropertyValue]()
		{
			AudioDevice->SetAudioStateProperty(PropertyName, PropertyValue);
		}, GET_STATID(STAT_SetAudioStateProperty));

		return;
	}

	FName& Property = AudioStateProperties.FindOrAdd(PropertyName);
	Property = PropertyValue;
}


FAudioDevice::FAudioSpatializationInterfaceInfo::FAudioSpatializationInterfaceInfo(FName InPluginName, FAudioDevice* InAudioDevice, IAudioSpatializationFactory* InAudioSpatializationFactoryPtr)
			: PluginName(InPluginName)
			, bIsInitialized(false)
{
	if(!ensure(InAudioDevice && InAudioSpatializationFactoryPtr))
	{
		return;
	}

	// use the factory to create a PluginInterface
	SpatializationPlugin = InAudioSpatializationFactoryPtr->CreateNewSpatializationPlugin(InAudioDevice);

	// cache metadata from the incoming plugin interface 
	bSpatializationIsExternalSend = InAudioSpatializationFactoryPtr->IsExternalSend();
	MaxChannelsSupportedBySpatializationPlugin = InAudioSpatializationFactoryPtr->GetMaxSupportedChannels();
	bReturnsToSubmixGraph = InAudioSpatializationFactoryPtr->ReturnsToSubmixGraph();
}

bool FAudioDevice::FAudioSpatializationInterfaceInfo::IsValid() const
{
	return SpatializationPlugin.IsValid() || !PluginName.IsNone();
}


bool FAudioDevice::ShouldUseAttenuation(const UWorld* World) const
{
	// We use attenuation settings:
	// - if we don't have a world, or
	// - we have a game world, or
	// - we are forcing the use of attenuation (e.g. for some editors)
	const bool bIsInGameWorld = World ? World->IsGameWorld() : true;
	return (bIsInGameWorld || bUseAttenuationForNonGameWorlds);
}

int32 FAudioDevice::GetNumPrecacheFrames() const
{
	// Check the cvar and use that if it's been set.
	if (NumPrecacheFramesCvar > 0)
	{
		return NumPrecacheFramesCvar;
	}
	// Otherwise, use the default value or value set in ini file
	return NumPrecacheFrames;
}

void FAudioDevice::UpdateUnpreparedSound(FWaveInstance* WaveInstance, bool bGameTicking) const
{
	// If this source is not playing yet due to a pending async decode, push it's active duration out 
	// by the amount of time that has elapsed.
	UpdateSoundDuration(WaveInstance, bGameTicking);
}

void FAudioDevice::UpdateSoundDuration(FWaveInstance* WaveInstance, bool bGameTicking) const
{
	if (FActiveSound* ActiveSound = WaveInstance->ActiveSound)
	{
		if (bGameTicking || ActiveSound->bIsUISound)
		{
			float ActiveDuration = ActiveSound->ComponentVolumeFader.GetActiveDuration();
			if (ActiveDuration > 0.0f)
			{
				float DeltaTime = GetGameDeltaTime();
				// Push the duration out by the previous frame's delta amount as the fader's elapsed time
				// has already been updated at this point with the same value.
				ActiveSound->ComponentVolumeFader.SetActiveDuration(ActiveDuration + DeltaTime);
			}
		}
	}
}
