// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioDeviceManager.h"

#include "Algo/Transform.h"
#include "Audio/AudioDebug.h"
#include "AudioAnalytics.h"
#include "AudioMixerDevice.h"
#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/GameUserSettings.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UObjectIterator.h"

#include "Sound/AudioFormatSettings.h"
#include "AudioDecompress.h"

#if INSTRUMENT_AUDIODEVICE_HANDLES
#include "Containers/StringConv.h"
#include "HAL/PlatformStackWalk.h"
#endif


#if WITH_EDITOR
#include "AudioEditorModule.h"
#include "Settings/LevelEditorMiscSettings.h"
#endif

// ENTRYPOINT
// AudioDeviceManager PreInit Callback, fired from Engine Startup Phase.
// This allows us to partially initialize early in the flow before assets start loading etc.
static FDelayedAutoRegisterHelper GAudioDeviceManagerPreInit(
	EDelayedRegisterRunPhase::IniSystemReady,
	&FAudioDeviceManager::PreInitialize);

static int32 GCVarEnableAudioThreadWait = 1;
TAutoConsoleVariable<int32> CVarEnableAudioThreadWait(
	TEXT("AudioThread.EnableAudioThreadWait"),
	GCVarEnableAudioThreadWait,
	TEXT("Enables waiting on the audio thread to finish its commands.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 CVarIsVisualizeEnabled = 0;
FAutoConsoleVariableRef CVarAudioVisualizeEnabled(
	TEXT("au.3dVisualize.Enabled"),
	CVarIsVisualizeEnabled,
	TEXT("Whether or not audio visualization is enabled. \n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 GCVarFlushAudioRenderCommandsOnSuspend = 0;
FAutoConsoleVariableRef CVarFlushAudioRenderCommandsOnSuspend(
	TEXT("au.FlushAudioRenderCommandsOnSuspend"),
	GCVarFlushAudioRenderCommandsOnSuspend,
	TEXT("When set to 1, ensures that we pump through all pending commands to the audio thread and audio render thread on app suspension.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 GCVarNeverMuteNonRealtimeAudioDevices = 0;
FAutoConsoleVariableRef CVarNeverMuteNonRealtimeAudioDevices(
	TEXT("au.NeverMuteNonRealtimeAudioDevices"),
	GCVarNeverMuteNonRealtimeAudioDevices,
	TEXT("When set to 1, nonrealtime audio devices will be exempt from normal audio device muting (for example, when a window goes out of focus.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

#if WITH_EDITOR
	static int32 GCVarForcePieClientsToShareAudio = 0;
	FAutoConsoleVariableRef CVarForcePieClientsToShareAudio(
		TEXT("au.ForcePieClientsToShareAudio"),
		GCVarForcePieClientsToShareAudio,
		TEXT("When set to 1, pie clients will always share audio devices instead of spawning a new one, ignoring the bCreateNewAudioDeviceForPlayInEditor setting."),
		ECVF_Default);
#endif

static FString AudioMixerModuleNameHotfixCVar;
static FString AudioMixerModuleNameHotfixCVarName = TEXT("au.AudioMixerModuleName");
FAutoConsoleVariableRef CVarAudioMixerModuleNameCVar(
	*AudioMixerModuleNameHotfixCVarName,
	AudioMixerModuleNameHotfixCVar,
	TEXT("Selects an audio mixer device for Microsoft platforms. \n")
	TEXT("AudioMixerWasapi or AudioMixerXAudio2"),
	ECVF_SaveForNextBoot);

static FAutoConsoleCommand GReportAudioDevicesCommand(
	TEXT("au.ReportAudioDevices"),
	TEXT("This will log any active audio devices (instances of the audio engine) alive right now."),
	FConsoleCommandDelegate::CreateStatic(
		[]()
	{
		FAudioDeviceManager::Get()->LogListOfAudioDevices();
	})
);


namespace AudioDeviceManagerUtils
{
	FString PrintDeviceInfo(const Audio::FDeviceId InDeviceId, const EAudioDeviceScope InScope, const bool bInIsNonRealtime, const int32* NumHandles = nullptr, const TMap<uint32, FString>* InStackWalk = nullptr)
	{
		FString ScopeStr;
		switch (InScope)
		{
			case EAudioDeviceScope::Shared:
				ScopeStr = TEXT("Shared");
				break;

			case EAudioDeviceScope::Unique:
				ScopeStr = TEXT("Unique");
				break;

			case EAudioDeviceScope::Default:
			default:
				ScopeStr = TEXT("Default");
				break;
		}

		FString DeviceInfo = FString::Printf(
			TEXT("                Id: %d, Scope: %s, Realtime: %s"),
			InDeviceId,
			*ScopeStr,
			bInIsNonRealtime ? TEXT("False") : TEXT("True"));

		if (!NumHandles)
		{
			return DeviceInfo;
		}

		DeviceInfo += FString::Printf(TEXT(", Num Handles: %d"), *NumHandles);

#if INSTRUMENT_AUDIODEVICE_HANDLES
		if (InStackWalk)
		{
			DeviceInfo += TEXT("\n            Active Handles:\n\n");
			for (const TPair<uint32, FString>& StackWalkPairs : *InStackWalk)
			{
				DeviceInfo += StackWalkPairs.Value;
				DeviceInfo += TEXT("\n\n");
			}
		}
#endif

		return DeviceInfo;
	}
}
FAudioDeviceManager* FAudioDeviceManager::Singleton = nullptr;

// Some stress tests:
#if INSTRUMENT_AUDIODEVICE_HANDLES
static TArray<FAudioDeviceHandle> IntentionallyLeakedHandles;

static FAutoConsoleCommand GLeakAudioDeviceCommand(
	TEXT("au.stresstest.LeakAnAudioDevice"),
	TEXT("This will intentionally leak a new audio device. Obviously, should only be used for testing."),
	FConsoleCommandDelegate::CreateStatic(
		[]()
{
	FAudioDeviceParams Params;
	Params.Scope = EAudioDeviceScope::Unique;
	IntentionallyLeakedHandles.Add(FAudioDeviceManager::Get()->RequestAudioDevice(Params));
})
);

static FAutoConsoleCommand GLeakAudioDeviceHandleCommand(
	TEXT("au.stresstest.LeakAnAudioDeviceHandle"),
	TEXT("This will intentionally leak a new handle to an audio device. Obviously, should only be used for testing."),
	FConsoleCommandDelegate::CreateStatic(
		[]()
{
	FAudioDeviceParams Params;
	Params.Scope = EAudioDeviceScope::Shared;
	IntentionallyLeakedHandles.Add(FAudioDeviceManager::Get()->RequestAudioDevice(Params));
})
);

static FAutoConsoleCommand GCleanUpAudioDeviceLeaksCommand(
	TEXT("au.stresstest.CleanUpAudioDeviceLeaks"),
	TEXT("Clean up any audio devices created through a leak command."),
	FConsoleCommandDelegate::CreateStatic(
		[]()
{
	IntentionallyLeakedHandles.Reset();
})
);
#endif

/*-----------------------------------------------------------------------------
FAudioDeviceManager implementation.
-----------------------------------------------------------------------------*/

bool FAudioDeviceManager::bEnableAggregateDeviceSupport = false;

FAudioDeviceManager::FAudioDeviceManager()
	: AudioDeviceModule(nullptr)
	, DeviceIDCounter(0)
	, NextResourceID(1)
	, SoloDeviceHandle(INDEX_NONE)
	, ActiveAudioDeviceID(INDEX_NONE)
	, bPlayAllDeviceAudio(false)
{

#if ENABLE_AUDIO_DEBUG
	AudioDebugger = MakeUnique<Audio::FAudioDebugger>();

	// Check for a command line debug sound argument.
	FString DebugSound;
	if (FParse::Value(FCommandLine::Get(), TEXT("DebugSound="), DebugSound))
	{
		GetDebugger().SetAudioDebugSound(*DebugSound);
	}

#endif //ENABLE_AUDIO_DEBUG
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FAudioDeviceManager::~FAudioDeviceManager()
{
	UE_LOGF(LogAudio, Display, "Beginning Audio Device Manager Shutdown (Module: %ls)...", *AudioMixerModuleName);

	TArray<Audio::FDeviceId> DeviceIds;
	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);
		Devices.GetKeys(DeviceIds);
	}

	if (DeviceIds.Num() > 0)
	{
		UE_LOGF(LogAudio, Display, "Destroying %d Remaining Audio Device(s)...", DeviceIds.Num());

		// Notify anyone listening to the device manager that we are about to destroy the audio device.
		for (Audio::FDeviceId DeviceId : DeviceIds)
		{
			// Fire pre-destroy first so async tasks (e.g. operator pool removals) can
			// be drained before OnAudioDeviceDestroyed subscribers attempt pool mutations.
			FAudioDeviceManagerDelegates::OnPreAudioDeviceDestroyed.Broadcast(DeviceId);
			FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Broadcast(DeviceId);
		}
	}

	FAudioThread::StopAudioThread();

	TMap<Audio::FDeviceId, FAudioDeviceContainer> DevicesToShutdown;
	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);
		DevicesToShutdown = MoveTemp(Devices);
	}

	// Can only be destroyed outside of critical section to avoid a deadlock,
	// but need to remove the device from the manager's list in case of calls
	// being executed from individual device render thread commands attempting
	// to access their given device. This is a means to communicate to pending
	// commands the device is no longer available without destroying it mid-flight.
	DevicesToShutdown.Reset();
	MainAudioDeviceHandle.Reset();

	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);

	InitPhase = EInitPhase::Constructed;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FAudioDevice* FAudioDeviceManager::GetAudioDeviceFromWorldContext(const UObject* WorldContextObject)
{
	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->GetNetMode() == NM_DedicatedServer)
	{
		return nullptr;
	}

	return ThisWorld->GetAudioDeviceRaw();
}

Audio::FMixerDevice* FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(const UObject* WorldContextObject)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFromWorldContext(WorldContextObject))
	{
		return static_cast<Audio::FMixerDevice*>(AudioDevice);
	}
	return nullptr;
}

IAudioDeviceModule* FAudioDeviceManager::GetAudioDeviceModule()
{
	return AudioDeviceModule;
}

FAudioDeviceParams FAudioDeviceManager::GetDefaultParamsForNewWorld()
{
	bool bCreateNewAudioDeviceForPlayInEditor = false;

#if WITH_EDITOR
	// GIsEditor is necessary here to ignore this setting for -game situations.
	if (GIsEditor)
	{
		bCreateNewAudioDeviceForPlayInEditor = GetDefault<ULevelEditorMiscSettings>()->bCreateNewAudioDeviceForPlayInEditor;
	}

	if (GCVarForcePieClientsToShareAudio)
	{
		bCreateNewAudioDeviceForPlayInEditor = false;
	}
#endif

	FAudioDeviceParams Params;
	Params.Scope = bCreateNewAudioDeviceForPlayInEditor ? EAudioDeviceScope::Unique : EAudioDeviceScope::Shared;

	return Params;
}

FAudioDeviceHandle FAudioDeviceManager::RequestAudioDevice(const FAudioDeviceParams& InParams)
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);
	
	// If the device class is not multiclient capable then fall back to sharing the device.
	// Note that this ignores the bCreateNewAudioDeviceForPlayInEditor editor pref.
	if (InParams.Scope == EAudioDeviceScope::Unique && AudioDeviceModule->IsAudioDeviceClassMulticlient())
	{
		return CreateNewDevice(InParams);
	}
	else
	{
		// See if we already have a device we can use.
		for (auto& Device : Devices)
		{
			if (CanUseAudioDevice(InParams, Device.Value))
			{
				RegisterWorld(InParams.AssociatedWorld, Device.Key);
				return BuildNewHandle(Device.Value, Device.Key, InParams);
			}
		}

		// If we did not find a suitable device, build one.
		return CreateNewDevice(InParams);
	}
}

void FAudioDeviceManager::RegisterWorld(UWorld* InWorld, Audio::FDeviceId DeviceId)
{
	if (!InWorld)
	{
		return;
	}

	if (FAudioDeviceContainer* DeviceContainer = Devices.Find(DeviceId))
	{
		if (!DeviceContainer->WorldsUsingThisDevice.Contains(InWorld))
		{
			UE_LOGF(LogAudio, Display, "Audio Device (ID: %d) registered with world '%ls'.", DeviceId, *InWorld->GetName());
			DeviceContainer->WorldsUsingThisDevice.AddUnique(InWorld);
			FAudioDeviceWorldDelegates::OnWorldRegisteredToAudioDevice.Broadcast(InWorld, DeviceId);
		}
	}
}

void FAudioDeviceManager::UnregisterWorld(UWorld* InWorld, Audio::FDeviceId DeviceId)
{
	if (!InWorld)
	{
		return;
	}

	if (FAudioDeviceContainer* DeviceContainer = Devices.Find(DeviceId))
	{
		if (DeviceContainer->WorldsUsingThisDevice.Contains(InWorld))
		{
			UE_LOGF(LogAudio, Display, "Audio Device unregistered from world '%ls'.", *InWorld->GetName());
			DeviceContainer->WorldsUsingThisDevice.Remove(InWorld);
			FAudioDeviceWorldDelegates::OnWorldUnregisteredWithAudioDevice.Broadcast(InWorld, DeviceId);
		}

		if (MainAudioDeviceHandle.World.Get() == InWorld)
		{
			MainAudioDeviceHandle.World.Reset();
		}
	}
}

void FAudioDeviceManager::RegisterAudioInfoFactories()
{
	// Load any Engine.ini defined modules necessary for registering format factories.
	TArray<FString> AudioInfoModules;
	if (!GConfig->GetArray(TEXT("Audio"), TEXT("AudioInfoModules"), AudioInfoModules, GEngineIni))
	{
		// If this is simply not defined, default to sensible list of internal formats.
		static const TCHAR* DefaultInfoModuleNames[] = {
			TEXT("OpusAudioDecoder"), TEXT("VorbisAudioDecoder"), TEXT("AdpcmAudioDecoder"), TEXT("BinkAudioDecoder"), TEXT("RadAudioDecoder")
		};
		AudioInfoModules.Append(DefaultInfoModuleNames);
		UE_LOGF(LogAudio, Warning, "Audio:AudioInfoModules is not defined, defaulting to built in formats. (%ls)",
			*FString::JoinBy(AudioInfoModules, TEXT(", "), [](const FString& i) { return i; }));
	}

	// Load any necessary audio modules.
	for (const FString& i : AudioInfoModules)
	{
		FModuleManager::Get().LoadModuleChecked(*i);
	}
		
	// Sanity check we have all the Factories we need to run now by 
	TArray<FName> AllFormats;
	GetAudioFormatSettings().GetAllWaveFormats(AllFormats);

	FString FailedFormatsString;
	int32 NumFailedFormats = 0;
	for (FName i : AllFormats)
	{
		if (!IAudioInfoFactoryRegistry::Get().Find(i))
		{
			FailedFormatsString += FString::Printf(TEXT("'%s' "), *i.ToString());
			NumFailedFormats++;
		}
	}
	checkf(NumFailedFormats == 0, TEXT("Failed to find these required AudioFormats: [ %s]"), *FailedFormatsString);
}

bool FAudioDeviceManager::PreInitializeManager()
{
	if (InitPhase == EInitPhase::Constructed)
	{
		// Register all formats
		AudioFormatSettings = MakePimpl<Audio::FAudioFormatSettings>(GConfig, GEngineIni, FPlatformProperties::IniPlatformName());
		RegisterAudioInfoFactories();
		InitPhase = EInitPhase::PreInitialized;
	}
	return InitPhase >= EInitPhase::PreInitialized;
}

bool FAudioDeviceManager::InitializeManager()
{
	// Do we also need to pre-init?
	if (InitPhase < EInitPhase::PreInitialized)
	{
		if (!PreInitializeManager())
		{
			return false;
		}
	}

	// Initialize if we need to...
	if (InitPhase == EInitPhase::PreInitialized )
	{	
		if (LoadDefaultAudioDeviceModule())
		{
			check(AudioDeviceModule);

			UAudioSettings* AudioSettings = GetMutableDefault<UAudioSettings>();
			check(AudioSettings);

			AudioSettings->LoadDefaultObjects();
			AudioSettings->RegisterParameterInterfaces();

			FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixer"));

#if WITH_EDITOR
			IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
			AudioEditorModule->RegisterEffectPresetAssetActions();
#endif

			FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FAudioDeviceManager::AppWillEnterBackground);

			InitPhase = EInitPhase::Initialized;
		}
	}

	return IsInitialized();
}

bool FAudioDeviceManager::CreateMainAudioDevice()
{
	if (!MainAudioDeviceHandle)
	{
		// Initialize the main audio device.
		FAudioDeviceParams MainDeviceParams;
		MainDeviceParams.Scope = EAudioDeviceScope::Shared;
		MainDeviceParams.bIsNonRealtime = false;
		MainDeviceParams.AssociatedWorld = GWorld;

		MainAudioDeviceHandle = RequestAudioDevice(MainDeviceParams);

		if (!MainAudioDeviceHandle)
		{
			UE_LOGF(LogAudio, Display, "Main audio device could not be initialized. Please check the value for AudioMixerModuleName in [Platform]Engine.ini.");
			return false;
		}

		if (GWorld)
		{
			GWorld->SetAudioDevice(MainAudioDeviceHandle);
		}

		FAudioThread::StartAudioThread();
	}
	return true;
}

bool FAudioDeviceManager::LoadDefaultAudioDeviceModule()
{
	check(!AudioDeviceModule);

	if (!AudioMixerModuleNameHotfixCVar.IsEmpty())
	{
		UE_LOGF(LogAudio, Display, "Current AudioMixerModuleName hotfixed to be: %ls", *AudioMixerModuleNameHotfixCVar);
		AudioMixerModuleName = AudioMixerModuleNameHotfixCVar;
	}

	bool bForceNonRealtimeRenderer = FParse::Param(FCommandLine::Get(), TEXT("DeterministicAudio"));
	
#if WITH_EDITOR
	// Check to see if the editor pref has been set. If not, fall back to engine setting.
	FString TempMixerModuleName;
	GConfig->GetString(TEXT("/Script/AudioEditor.AudioEditorSettings"), TEXT("AudioMixerModuleName"), TempMixerModuleName, GEditorSettingsIni);
	if (!TempMixerModuleName.IsEmpty())
	{
		AudioMixerModuleName = TempMixerModuleName;
	}
	
#endif //WITH_EDITOR

	if (AudioMixerModuleName.IsEmpty())
	{
		// If not using command line switch to use audio mixer, check the game platform engine ini file (e.g. WindowsEngine.ini) which enables it for player
		GConfig->GetString(TEXT("Audio"), TEXT("AudioMixerModuleName"), AudioMixerModuleName, GEngineIni);
	}

	if (bForceNonRealtimeRenderer)
	{
		AudioDeviceModule = FModuleManager::LoadModulePtr<IAudioDeviceModule>(TEXT("NonRealtimeAudioRenderer"));
		return AudioDeviceModule != nullptr;
	}

	if (AudioMixerModuleName.Len() > 0)
	{
		AudioDeviceModule = FModuleManager::LoadModulePtr<IAudioDeviceModule>(*AudioMixerModuleName);
	}

	return AudioDeviceModule != nullptr;
}

FAudioDeviceHandle FAudioDeviceManager::CreateNewDevice(const FAudioDeviceParams& InParams)
{
	const Audio::FDeviceId DeviceID = GetNewDeviceID();

	FString DeviceInfo = AudioDeviceManagerUtils::PrintDeviceInfo(DeviceID, InParams.Scope, InParams.bIsNonRealtime);
	UE_LOGF(LogAudio, Display, "Creating Audio Device: %ls", *DeviceInfo);
	Devices.Emplace(DeviceID, FAudioDeviceContainer(InParams, DeviceID, this));

	FAudioDeviceContainer* ContainerPtr = Devices.Find(DeviceID);
	check(ContainerPtr);
	if (!ContainerPtr->Device)
	{
		UE_LOGF(LogAudio, Display, "Destroying Audio Device %d: could not be initialized. Check AudioMixerModuleName in [Platform]Engine.ini.", DeviceID);

		// Initializing the audio device failed. Remove the device container and return an empty handle.
		Devices.Remove(DeviceID);
		return FAudioDeviceHandle();
	}
	else
	{
		RegisterWorld(InParams.AssociatedWorld, DeviceID);

		FAudioDeviceHandle Handle = BuildNewHandle(*ContainerPtr, DeviceID, InParams);
		FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Broadcast(DeviceID);
		return Handle;
	}
}

bool FAudioDeviceManager::IsValidAudioDevice(Audio::FDeviceId Handle) const
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);

	return Devices.Contains(Handle);
}

void FAudioDeviceManager::IncrementDevice(Audio::FDeviceId DeviceID)
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);

	// If there is an FAudioDeviceHandle out in the world
	check(Devices.Contains(DeviceID));

	FAudioDeviceContainer& Container = Devices[DeviceID];
	Container.NumberOfHandlesToThisDevice++;
}

void FAudioDeviceManager::DecrementDevice(Audio::FDeviceId DeviceID, UWorld* InWorld)
{
	FAudioDevice* DeviceToTearDown = nullptr;

	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);

		// If there is an FAudioDeviceHandle out in the world
		if (Devices.Contains(DeviceID))
		{
			FAudioDeviceContainer& Container = Devices[DeviceID];
			check(Container.NumberOfHandlesToThisDevice > 0);

			// Report device being destroyed before actual destruction
			// to allow listeners to access and respond where applicable.
			bool bDestroyingDevice = false;
			if (Container.NumberOfHandlesToThisDevice == 1)
			{
				bDestroyingDevice = true;
				// Fire pre-destroy first so async tasks (e.g. operator pool removals) can
				// be drained before OnAudioDeviceDestroyed subscribers attempt pool mutations.
				FAudioDeviceManagerDelegates::OnPreAudioDeviceDestroyed.Broadcast(DeviceID);
				FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Broadcast(DeviceID);

				// Subsystems deinitialization
				Container.Device->Deinitialize();

				// If this is the active device and being destroyed, set the main device as the active device.
				if (DeviceID == ActiveAudioDeviceID)
				{
					SetActiveDevice(MainAudioDeviceHandle.GetDeviceID());
				}

				UnregisterWorld(InWorld, DeviceID);
			}

			Container.NumberOfHandlesToThisDevice--;

			// If there is no longer any users of this device, destroy it.
			if (Container.NumberOfHandlesToThisDevice)
			{
				ensureMsgf(!bDestroyingDevice, TEXT("AudioDevice Destruction Failure: 'OnAudioDeviceDestroyed' listener generated new persistent handle(s) to AudioDevice."));
			}
			else
			{
				Swap(DeviceToTearDown, Container.Device);
				Devices.Remove(DeviceID);
			}
		}
	}

	if (DeviceToTearDown)
	{
		DeviceToTearDown->FadeOut();
		DeviceToTearDown->Teardown();
		delete DeviceToTearDown;
	}
}

FAudioDeviceHandle FAudioDeviceManager::BuildNewHandle(FAudioDeviceContainer&Container, Audio::FDeviceId DeviceID, const FAudioDeviceParams &InParams)
{
	FAudioDeviceManager::Get()->IncrementDevice(DeviceID);
	return FAudioDeviceHandle(Container.Device, DeviceID, InParams.AssociatedWorld);
}

bool FAudioDeviceManager::CanUseAudioDevice(const FAudioDeviceParams& InParams, const FAudioDeviceContainer& InContainer)
{
	return InContainer.Scope == EAudioDeviceScope::Shared
		&& InParams.AudioModule == InContainer.SpecifiedModule
		&& InParams.bIsNonRealtime == InContainer.bIsNonRealtime;
}

#if INSTRUMENT_AUDIODEVICE_HANDLES
uint32 FAudioDeviceManager::CreateUniqueStackWalkID()
{
	static uint32 UniqueStackWalkID = 0;
	return UniqueStackWalkID++;
}
#endif

FAudioDeviceHandle FAudioDeviceManager::GetAudioDevice(Audio::FDeviceId InDeviceID)
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);
	FAudioDeviceContainer* Container = Devices.Find(InDeviceID);

	if (Container)
	{
		FAudioDeviceParams Params = FAudioDeviceParams();
		return BuildNewHandle(*Container, InDeviceID, Params);
	}
	else
	{
		return FAudioDeviceHandle();
	}
}

FAudioDevice* FAudioDeviceManager::GetAudioDeviceRaw(Audio::FDeviceId InDeviceID) 
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);
	if (!IsValidAudioDevice(InDeviceID))
	{
		return nullptr;
	}

	FAudioDevice* AudioDevice = Devices[InDeviceID].Device;
	check(AudioDevice != nullptr);

	return AudioDevice;
}

const FAudioDevice* FAudioDeviceManager::GetAudioDeviceRaw(Audio::FDeviceId InDeviceID) const
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);
	if (!IsValidAudioDevice(InDeviceID))
	{
		return nullptr;
	}

	const FAudioDevice* AudioDevice = Devices[InDeviceID].Device;
	check(AudioDevice != nullptr);

	return AudioDevice;
}

void FAudioDeviceManager::SetAudioDevice(UWorld& InWorld, Audio::FDeviceId InDeviceID)
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);
	{
		if (FAudioDeviceContainer* Container = Devices.Find(InDeviceID))
		{
			FAudioDeviceParams Params = FAudioDeviceParams();
			Params.AssociatedWorld = &InWorld;
			FAudioDeviceHandle Handle = BuildNewHandle(*Container, InDeviceID, Params);
			InWorld.SetAudioDevice(Handle);
		}
		else
		{
			static const FAudioDeviceHandle InvalidHandle;
			InWorld.SetAudioDevice(InvalidHandle);
		}
	}
}

bool FAudioDeviceManager::PreInitialize()
{
	// (Optionally) Pre-Initialize the AudioDeviceManager.
	// By pre-initialing the Audio Device Manager we can start up some low level services needed for IO ahead of the main init.
	// NOTE: Calling Get() will still return null until the device is initialized fully.

	if (FAudioDeviceManager* Adm = GetOrCreate())
	{
		UE_LOGF(LogAudio, Display, "Pre-Initializing Audio Device Manager...");
		if (Adm->PreInitializeManager())
		{
			UE_LOGF(LogAudio, Display, "Audio Device Manager Pre-Initialized");
		}
		else
		{
			UE_LOGF(LogAudio, Warning, "Audio Device Manager Pre-Initialization Failed!");
			delete Adm;
			Singleton = nullptr;
		}
	}
	return Singleton && Singleton->InitPhase >= EInitPhase::PreInitialized;
}

bool FAudioDeviceManager::Initialize()
{
	if (FAudioDeviceManager* Adm = GetOrCreate())
	{
		UE_LOGF(LogAudio, Display, "Initializing Audio Device Manager...");
		if (Adm->InitializeManager())
		{
			UE_LOGF(LogAudio, Display, "Audio Device Manager Initialized");
		}
		else
		{
			UE_LOGF(LogAudio, Warning, "Audio Device Manager Initialization Failed!");			
			delete Adm;
			Singleton = nullptr;
		}
	}
	return Singleton && Singleton->IsInitialized();
}

FAudioDeviceManager* FAudioDeviceManager::Get()
{
	if (Singleton && Singleton->IsInitialized())
	{
		return Singleton;
	}
	return nullptr;
}

FAudioDeviceManager* FAudioDeviceManager::GetOrCreate()
{
	if (!Singleton)
	{
		if (FApp::CanEverRenderAudio())
		{
			Singleton = new FAudioDeviceManager();
		}
		else
		{
			static bool bDoOnce = false;
			if (!bDoOnce)
			{
				UE_LOGF(LogAudio, Display, "Audio Device Manager not initializing due to all audio being disabled. If this is not intentional, please check command line arguments for \"-nosound\".");

				Audio::Analytics::RecordEvent_Usage(TEXT("AllAudioDisabled"));
				bDoOnce = true;
			}
		}
	}
	return Singleton;
}
void FAudioDeviceManager::Shutdown()
{
	if (Singleton)
	{
		delete Singleton;
		Singleton = nullptr;
		UE_LOGF(LogAudio, Display, "Audio Device Manager Shutdown");
	}
}

FAudioDeviceHandle FAudioDeviceManager::GetActiveAudioDevice()
{
	if (ActiveAudioDeviceID != INDEX_NONE)
	{
		FAudioDeviceHandle ActiveAudioDeviceHandle = GetAudioDevice(ActiveAudioDeviceID);

		if (ActiveAudioDeviceHandle)
		{
			return ActiveAudioDeviceHandle;
		}
	}

	return MainAudioDeviceHandle;
}

void FAudioDeviceManager::UpdateActiveAudioDevices(bool bGameTicking)
{
	// Before we kick off the next update make sure that we've finished the previous frame's update (this should be extremely rare)
	if (GCVarEnableAudioThreadWait)
	{
		SyncFence.Wait();
	}

	IterateOverAllDevices(
		[&bGameTicking](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->Update(bGameTicking);
		}
	);

	if (GCVarEnableAudioThreadWait)
	{
		SyncFence.BeginFence();
	}
}

void FAudioDeviceManager::IterateOverAllDevices(TUniqueFunction<void(Audio::FDeviceId, FAudioDevice*)> ForEachDevice)
{
	TArray<Audio::FDeviceId> DeviceIDs;
	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);
		Devices.GetKeys(DeviceIDs);
	}

	for (const Audio::FDeviceId DeviceID : DeviceIDs)
	{
		FAudioDeviceHandle DeviceHandle = GetAudioDevice(DeviceID);
		if (DeviceHandle.IsValid())
		{
			ForEachDevice(DeviceID, DeviceHandle.GetAudioDevice());
		}
	}
}

void FAudioDeviceManager::IterateOverAllDevices(TUniqueFunction<void(Audio::FDeviceId, const FAudioDevice*)> ForEachDevice) const
{
	TArray<Audio::FDeviceId> DeviceIDs;
	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);
		Devices.GetKeys(DeviceIDs);
	}

	for (const Audio::FDeviceId DeviceID : DeviceIDs)
	{
		if (const FAudioDevice* Device = GetAudioDeviceRaw(DeviceID))
		{
			ForEachDevice(DeviceID, Device);
		}
	}
}

void FAudioDeviceManager::AddReferencedObjects(FReferenceCollector& Collector)
{
#if !WITH_EDITORONLY_DATA
	// Audio object references are updated while the audio thread is running and so
	// we need to make sure that the audio thread is not running while we collect
	// object references. The AudioThread generally only exists in packaged games
	// and it has mechanisms for pausing/resuming. It is triggered by the start 
	// and stop of GarbageCollection and so for most scenarios we are safe here. 
	//
	// One exception to the rule is StandaloneGame which launches the AudioThread 
	// but is not packaged. There are several BP fixup and redirector calls that
	// occur on level load which hit this callstack even though they are not part
	// of GarbageCollection. This inherent race condition has existed for at least
	// 10 years and has not caused known issues. It remains as tech-debt. 
	
	//checkf(!IsAudioThreadRunning(), TEXT("The audio thread must be disabled or suspended while collecting object references"));
	// This check should be renabled with the resolution of UE-253226
#endif

	IterateOverAllDevices(
		[&Collector](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->AddReferencedObjects(Collector);
		}
	);
}

void FAudioDeviceManager::StopSoundsUsingResource(USoundWave* InSoundWave, TArray<UAudioComponent*>* StoppedComponents)
{
	IterateOverAllDevices(
		[&InSoundWave, &StoppedComponents](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->StopSoundsUsingResource(InSoundWave, StoppedComponents);
		}
	);
}

void FAudioDeviceManager::RegisterSoundClass(USoundClass* SoundClass)
{
	IterateOverAllDevices(
		[&SoundClass](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->RegisterSoundClass(SoundClass);
		}
	);
}

void FAudioDeviceManager::UnregisterSoundClass(USoundClass* SoundClass)
{
	IterateOverAllDevices(
		[&SoundClass](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->UnregisterSoundClass(SoundClass);
		}
	);
}

void FAudioDeviceManager::InitSoundClasses()
{
	IterateOverAllDevices(
		[](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->InitSoundClasses();
		}
	);
}

void FAudioDeviceManager::RegisterSoundSubmix(USoundSubmixBase* SoundSubmix)
{
	IterateOverAllDevices(
		[&SoundSubmix](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->RegisterSoundSubmix(SoundSubmix, true);
		}
	);
}

void FAudioDeviceManager::UnregisterSoundSubmix(const USoundSubmixBase* SoundSubmix)
{
	IterateOverAllDevices(
		[&SoundSubmix](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->UnregisterSoundSubmix(SoundSubmix, true);
		}
	);
}

void FAudioDeviceManager::InitSoundSubmixes()
{
	IterateOverAllDevices(
		[](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->InitSoundSubmixes();
		}
	);
}

void FAudioDeviceManager::InitSoundEffectPresets()
{
	// Deprecated.
}

void FAudioDeviceManager::UpdateSourceEffectChain(const uint32 SourceEffectChainId, const TArray<FSourceEffectChainEntry>& SourceEffectChain, const bool bPlayEffectChainTails)
{
	IterateOverAllDevices(
		[&SourceEffectChainId, &SourceEffectChain, &bPlayEffectChainTails](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->UpdateSourceEffectChain(SourceEffectChainId, SourceEffectChain, bPlayEffectChainTails);
		}
	);
}

void FAudioDeviceManager::UpdateSubmix(USoundSubmixBase* SoundSubmix)
{
	IterateOverAllDevices(
		[&SoundSubmix](Audio::FDeviceId, FAudioDevice* InDevice)
		{
			InDevice->UpdateSubmixProperties(SoundSubmix);
		}
	);
}

void FAudioDeviceManager::SetActiveDevice(uint32 InAudioDeviceHandle)
{
	// Only change the active device if there are no solo'd audio devices
	if (SoloDeviceHandle == INDEX_NONE)
	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);
		// Iterate over all of our devices and mute every device except for InAudioDeviceHandle:
		for (auto& DeviceContainer : Devices)
		{
			check(DeviceContainer.Value.Device);
			FAudioDevice* AudioDevice = DeviceContainer.Value.Device;

			if (DeviceContainer.Key == InAudioDeviceHandle)
			{
				ActiveAudioDeviceID = InAudioDeviceHandle;
				AudioDevice->SetDeviceMuted(false);
			}
			else
			{
				AudioDevice->SetDeviceMuted(true);
			}
		}
	}
}

void FAudioDeviceManager::SetSoloDevice(Audio::FDeviceId InAudioDeviceHandle)
{
	SoloDeviceHandle = InAudioDeviceHandle;
	if (SoloDeviceHandle != INDEX_NONE)
	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);
		for (auto& DeviceContainer : Devices)
		{
			check(DeviceContainer.Value.Device);
			check(DeviceContainer.Key == DeviceContainer.Value.Device->DeviceID);
			FAudioDevice*& AudioDevice = DeviceContainer.Value.Device;

			// Un-mute the active audio device and mute non-active device, as long as its not the main audio device (which is used to play UI sounds)
			if (AudioDevice->DeviceID == InAudioDeviceHandle)
			{
				ActiveAudioDeviceID = InAudioDeviceHandle;
				AudioDevice->SetDeviceMuted(false);
			}
			else
			{
				AudioDevice->SetDeviceMuted(true);
			}
		}
	}
}


uint8 FAudioDeviceManager::GetNumActiveAudioDevices() const
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);
	return Devices.Num();
}

uint8 FAudioDeviceManager::GetNumMainAudioDeviceWorlds() const
{
	FScopeLock ScopeLock(&DeviceMapCriticalSection);

	const Audio::FDeviceId MainDeviceID = MainAudioDeviceHandle.GetDeviceID();
	if (Devices.Contains(MainDeviceID))
	{
		return Devices[MainDeviceID].WorldsUsingThisDevice.Num();
	}
	else
	{
		return 0;
	}
}

TArray<FAudioDevice*> FAudioDeviceManager::GetAudioDevices() const
{
	TArray<FAudioDevice*> DeviceList;

	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);
		Algo::Transform(Devices, DeviceList, [](const TPair<Audio::FDeviceId, FAudioDeviceContainer>& Pair)
		{
			return Pair.Value.Device;
		});
	}

	return DeviceList;
}

TArray<UWorld*> FAudioDeviceManager::GetWorldsUsingAudioDevice(const Audio::FDeviceId& InID) const
{
	{
		FScopeLock ScopeLock(&DeviceMapCriticalSection);
		if (Devices.Contains(InID))
		{
			return Devices[InID].WorldsUsingThisDevice;
		}
	}

	return { };
}

#if INSTRUMENT_AUDIODEVICE_HANDLES
void FAudioDeviceManager::AddStackWalkForContainer(Audio::FDeviceId InId, uint32 StackWalkID, FString&& InStackWalk)
{
	check(Devices.Contains(InId));
	check(!Devices[InId].HandleCreationStackWalks.Contains(StackWalkID));
	FAudioDeviceContainer& Container = Devices[InId];
	Container.HandleCreationStackWalks.Add(StackWalkID, MoveTemp(InStackWalk));
}

void FAudioDeviceManager::RemoveStackWalkForContainer(Audio::FDeviceId InId, uint32 StackWalkID)
{
	if (!Devices.Contains(InId))
	{
		return;
	}

	check(Devices[InId].HandleCreationStackWalks.Contains(StackWalkID));
	FAudioDeviceContainer& Container = Devices[InId];
	Container.HandleCreationStackWalks.Remove(StackWalkID);
}
#endif

void FAudioDeviceManager::LogListOfAudioDevices()
{
	FString ListOfDevices;

	for (const TPair<Audio::FDeviceId, FAudioDeviceContainer>& DevicePair : Devices)
	{
		ListOfDevices += AudioDeviceManagerUtils::PrintDeviceInfo(
			DevicePair.Key,
			DevicePair.Value.Scope,
			DevicePair.Value.bIsNonRealtime,
			&DevicePair.Value.NumberOfHandlesToThisDevice
#if INSTRUMENT_AUDIODEVICE_HANDLES
			, &DevicePair.Value.HandleCreationStackWalks
#endif
		);
	}

	UE_LOGF(LogAudio, Display, "Active Audio Devices:\n%ls", *ListOfDevices);
}

Audio::FAudioFormatSettings& FAudioDeviceManager::GetAudioFormatSettings() const
{
	check(AudioFormatSettings.IsValid());
	return *AudioFormatSettings;
}

uint32 FAudioDeviceManager::GetNewDeviceID()
{
	return ++DeviceIDCounter;
}

// (deprecated)
void FAudioDeviceManager::StopSourcesUsingBuffer(FSoundBuffer*)
{
}

// (deprecated)
void FAudioDeviceManager::TrackResource(USoundWave* SoundWave, FSoundBuffer* Buffer)
{
	// Allocate new resource ID and assign to USoundWave. A value of 0 (default) means not yet registered.
	int32 ResourceID = NextResourceID++;
	Buffer->ResourceID = ResourceID;
	SoundWave->ResourceID = ResourceID;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Keep track of associated resource name.
	Buffer->ResourceName = SoundWave->GetPathName();
#endif
}

void FAudioDeviceManager::FreeResource(USoundWave* SoundWave)
{
	if (SoundWave->ResourceID)
	{
		// Flag that the sound wave needs to do a full decompress again
		SoundWave->DecompressionType = DTYPE_Setup;
		SoundWave->SetPrecacheState(ESoundWavePrecacheState::NotStarted);

		SoundWave->ResourceID = 0;
	}
}

// (deprecated)
void FAudioDeviceManager::FreeBufferResource(FSoundBuffer* SoundBuffer)
{
	if (SoundBuffer)
	{
		// Make sure any realtime tasks are finished that are using this buffer
		SoundBuffer->EnsureRealtimeTaskCompletion();
		delete SoundBuffer;
		SoundBuffer = nullptr;
	}
}

// (deprecated)
FSoundBuffer* FAudioDeviceManager::GetSoundBufferForResourceID(uint32 ResourceID)
{
	return {};
}

// deprecated
void FAudioDeviceManager::RemoveSoundBufferForResourceID(uint32)
{
}

void FAudioDeviceManager::RemoveSoundMix(USoundMix* SoundMix)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.RemoveSoundMix"), STAT_AudioRemoveSoundMix, STATGROUP_AudioThreadCommands);

		FAudioDeviceManager* AudioDeviceManager = this;
		FAudioThread::RunCommandOnAudioThread([AudioDeviceManager, SoundMix]()
		{
			AudioDeviceManager->RemoveSoundMix(SoundMix);

		}, GET_STATID(STAT_AudioRemoveSoundMix));

		return;
	}

	IterateOverAllDevices([SoundMix](Audio::FDeviceId Id, FAudioDevice* Device)
	{
		Device->RemoveSoundMix(SoundMix);
	});
}

void FAudioDeviceManager::TogglePlayAllDeviceAudio()
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.TogglePlayAllDeviceAudio"), STAT_TogglePlayAllDeviceAudio, STATGROUP_AudioThreadCommands);

		FAudioDeviceManager* AudioDeviceManager = this;
		FAudioThread::RunCommandOnAudioThread([AudioDeviceManager]()
		{
			AudioDeviceManager->TogglePlayAllDeviceAudio();

		}, GET_STATID(STAT_TogglePlayAllDeviceAudio));

		return;
	}

	bPlayAllDeviceAudio = !bPlayAllDeviceAudio;
}

bool FAudioDeviceManager::IsAlwaysPlayNonRealtimeDeviceAudio() const
{
	return GCVarNeverMuteNonRealtimeAudioDevices != 0;
}

bool FAudioDeviceManager::IsVisualizeDebug3dEnabled() const
{
#if ENABLE_AUDIO_DEBUG
	return GetDebugger().IsVisualizeDebug3dEnabled() || CVarIsVisualizeEnabled;
#else // ENABLE_AUDIO_DEBUG
	return false;
#endif // !ENABLE_AUDIO_DEBUG
}

void FAudioDeviceManager::ToggleVisualize3dDebug()
{
#if ENABLE_AUDIO_DEBUG
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ToggleVisualize3dDebug"), STAT_ToggleVisualize3dDebug, STATGROUP_AudioThreadCommands);

		FAudioDeviceManager* AudioDeviceManager = this;
		FAudioThread::RunCommandOnAudioThread([AudioDeviceManager]()
		{
			AudioDeviceManager->ToggleVisualize3dDebug();

		}, GET_STATID(STAT_ToggleVisualize3dDebug));

		return;
	}

	GetDebugger().ToggleVisualizeDebug3dEnabled();
#endif // ENABLE_AUDIO_DEBUG
}

float FAudioDeviceManager::GetDynamicSoundVolume(ESoundType SoundType, const FName& SoundName) const
{
	check(IsInAudioThread());

	TTuple<ESoundType, FName> SoundKey(SoundType, SoundName);
	if (const float* Volume = DynamicSoundVolumes.Find(SoundKey))
	{
		return FMath::Max(0.0f, *Volume);
	}

	return 1.0f;
}

void FAudioDeviceManager::ResetAllDynamicSoundVolumes()
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ResetAllDynamicSoundVolumes"), STAT_ResetAllDynamicSoundVolumes, STATGROUP_AudioThreadCommands);

		FAudioDeviceManager* AudioDeviceManager = this;
		FAudioThread::RunCommandOnAudioThread([AudioDeviceManager]()
		{
			AudioDeviceManager->ResetAllDynamicSoundVolumes();

		}, GET_STATID(STAT_ResetAllDynamicSoundVolumes));
		return;
	}

	DynamicSoundVolumes.Reset();
	DynamicSoundVolumes.Shrink();
}

void FAudioDeviceManager::ResetDynamicSoundVolume(ESoundType SoundType, const FName& SoundName)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ResetSoundCueTrimVolume"), STAT_ResetSoundCueTrimVolume, STATGROUP_AudioThreadCommands);

		FAudioDeviceManager* AudioDeviceManager = this;
		FAudioThread::RunCommandOnAudioThread([AudioDeviceManager, SoundType, SoundName]()
		{
			AudioDeviceManager->ResetDynamicSoundVolume(SoundType, SoundName);

		}, GET_STATID(STAT_ResetSoundCueTrimVolume));
		return;
	}

	TTuple<ESoundType, FName> Key(SoundType, SoundName);
	DynamicSoundVolumes.Remove(Key);
}

void FAudioDeviceManager::SetDynamicSoundVolume(ESoundType SoundType, const FName& SoundName, float Volume)
{
	if (!IsInAudioThread())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetDynamicSoundVolume"), STAT_SetDynamicSoundVolume, STATGROUP_AudioThreadCommands);

		FAudioDeviceManager* AudioDeviceManager = this;
		FAudioThread::RunCommandOnAudioThread([AudioDeviceManager, SoundType, SoundName, Volume]()
		{
			AudioDeviceManager->SetDynamicSoundVolume(SoundType, SoundName, Volume);

		}, GET_STATID(STAT_SetDynamicSoundVolume));
		return;
	}

	TTuple<ESoundType, FName> Key(SoundType, SoundName);
	DynamicSoundVolumes.FindOrAdd(Key) = FMath::Clamp(Volume, 0.0f, MAX_VOLUME);;
}

void FAudioDeviceManager::EnableAggregateDeviceSupport(bool bInEnableAggregateDevice)
{
	bEnableAggregateDeviceSupport = bInEnableAggregateDevice;
}

bool FAudioDeviceManager::IsAggregateDeviceSupportEnabled()
{
	return bEnableAggregateDeviceSupport;
}

#if ENABLE_AUDIO_DEBUG
Audio::FAudioDebugger& FAudioDeviceManager::GetDebugger()
{
	check(AudioDebugger.IsValid());

	return *AudioDebugger;
}

const Audio::FAudioDebugger& FAudioDeviceManager::GetDebugger() const
{
	check(AudioDebugger.IsValid());

	return *AudioDebugger;
}

#endif // ENABLE_AUDIO_DEBUG


void FAudioDeviceManager::AppWillEnterBackground()
{
	SCOPED_ENTER_BACKGROUND_EVENT(STAT_FAudioDeviceManager_AppWillEnterBackground);

	// Flush all commands to the audio thread and the audio render thread:
	if (GCVarFlushAudioRenderCommandsOnSuspend)
	{
		if (MainAudioDeviceHandle.IsValid())
		{
			FAudioThread::RunCommandOnAudioThread([this]()
			{
				if (MainAudioDeviceHandle.IsValid())
				{
					MainAudioDeviceHandle->FlushAudioRenderingCommands(true);
				}
			}, TStatId());
		}

		FAudioCommandFence AudioCommandFence;
		AudioCommandFence.BeginFence();
		AudioCommandFence.Wait();
	}
}

FAudioDeviceHandle::FAudioDeviceHandle()
	: World(nullptr)
	, Device(nullptr)
	, DeviceId(INDEX_NONE)
{
#if INSTRUMENT_AUDIODEVICE_HANDLES
	StackWalkID = INDEX_NONE;
#endif
}

FAudioDeviceHandle::FAudioDeviceHandle(FAudioDevice* InDevice, Audio::FDeviceId InID, UWorld* InWorld)
	: World(InWorld)
	, Device(InDevice)
	, DeviceId(InID)
{
#if INSTRUMENT_AUDIODEVICE_HANDLES
	AddStackDumpToAudioDeviceContainer();
#endif
}

FAudioDeviceHandle::FAudioDeviceHandle(const FAudioDeviceHandle& Other)
	: FAudioDeviceHandle()
{
	*this = Other;
}

FAudioDeviceHandle::FAudioDeviceHandle(FAudioDeviceHandle&& Other)
	: FAudioDeviceHandle()
{
	*this = MoveTemp(Other);
}

#if INSTRUMENT_AUDIODEVICE_HANDLES
void FAudioDeviceHandle::AddStackDumpToAudioDeviceContainer()
{
	static const int32 MaxPlatformWalkStringCount = 1024 * 4;

	ANSICHAR PlatformDump[MaxPlatformWalkStringCount];
	FMemory::Memzero(PlatformDump, MaxPlatformWalkStringCount * sizeof(ANSICHAR));

	FPlatformStackWalk::StackWalkAndDump(PlatformDump, MaxPlatformWalkStringCount - 1, 2);

	FString FormattedDump = "New Handle Created:\n";

	int32 DumpLength = FCStringAnsi::Strlen(PlatformDump);

	// If this hits, increase the max character length.
	ensure(DumpLength < MaxPlatformWalkStringCount - 1);

	FormattedDump.AppendChars(ANSI_TO_TCHAR(PlatformDump), DumpLength);
	FormattedDump += TEXT("\n");
	StackWalkID = FAudioDeviceManager::Get()->CreateUniqueStackWalkID();
	FAudioDeviceManager::Get()->AddStackWalkForContainer(DeviceId, StackWalkID, MoveTemp(FormattedDump));
}
#endif

FAudioDeviceHandle::~FAudioDeviceHandle()
{
	if (IsValid())
	{
		FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (ensure(AudioDeviceManager))
		{
			AudioDeviceManager->DecrementDevice(DeviceId, World.Get());

#if INSTRUMENT_AUDIODEVICE_HANDLES
			check(StackWalkID != INDEX_NONE);
			AudioDeviceManager->RemoveStackWalkForContainer(DeviceId, StackWalkID);
#endif
		}
	}
}

FAudioDevice* FAudioDeviceHandle::GetAudioDevice() const
{
	return Device;
}

TWeakObjectPtr<UWorld> FAudioDeviceHandle::GetWorld() const
{
	return World;
}

Audio::FDeviceId FAudioDeviceHandle::GetDeviceID() const
{
	return DeviceId;
}

bool FAudioDeviceHandle::IsValid() const
{
	return Device != nullptr;
}

void FAudioDeviceHandle::Reset()
{
	*this = FAudioDeviceHandle();
}

FAudioDeviceHandle& FAudioDeviceHandle::operator=(const FAudioDeviceHandle& Other)
{
	const bool bWasValid = IsValid();
	const Audio::FDeviceId OldDeviceId = DeviceId;
	UWorld* OldWorld = World.Get();

#if INSTRUMENT_AUDIODEVICE_HANDLES
	const uint32 OldStackWalkID = StackWalkID;
#endif

	Device = Other.Device;
	DeviceId = Other.DeviceId;
	World = Other.World;

	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		if (IsValid())
		{
			AudioDeviceManager->IncrementDevice(DeviceId);

#if INSTRUMENT_AUDIODEVICE_HANDLES
			AddStackDumpToAudioDeviceContainer();
#endif
		}

		if (bWasValid)
		{
			AudioDeviceManager->DecrementDevice(OldDeviceId, OldWorld);

#if INSTRUMENT_AUDIODEVICE_HANDLES
			check(OldStackWalkID != INDEX_NONE);
			AudioDeviceManager->RemoveStackWalkForContainer(OldDeviceId, OldStackWalkID);
#endif
		}
	}

	return *this;
}

FAudioDeviceHandle& FAudioDeviceHandle::operator=(FAudioDeviceHandle&& Other)
{
#if INSTRUMENT_AUDIODEVICE_HANDLES
	const uint32 OldStackWalkID = StackWalkID;
#endif

	const bool bWasValid = IsValid();
	const Audio::FDeviceId OldDeviceId = DeviceId;
	UWorld* OldWorld = World.Get();

	Device = Other.Device;
	DeviceId = Other.DeviceId;
	World = Other.World;

	FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
	if (AudioDeviceManager && IsValid())
	{
#if INSTRUMENT_AUDIODEVICE_HANDLES
		AddStackDumpToAudioDeviceContainer();
#endif
	}

	if (AudioDeviceManager && bWasValid)
	{
#if INSTRUMENT_AUDIODEVICE_HANDLES
		check(OldStackWalkID != INDEX_NONE);
		AudioDeviceManager->RemoveStackWalkForContainer(OldDeviceId, OldStackWalkID);
#endif

		AudioDeviceManager->DecrementDevice(OldDeviceId, OldWorld);
	}

	if (AudioDeviceManager && Other.IsValid())
	{
#if INSTRUMENT_AUDIODEVICE_HANDLES
		check(Other.StackWalkID != INDEX_NONE);
		AudioDeviceManager->RemoveStackWalkForContainer(Other.DeviceId, Other.StackWalkID);
#endif
	}

	Other.Device = nullptr;
	Other.DeviceId = INDEX_NONE;
	Other.World.Reset();

#if INSTRUMENT_AUDIODEVICE_HANDLES
	Other.StackWalkID = INDEX_NONE;
#endif

	return *this;
}

FAudioDeviceManager::FAudioDeviceContainer::FAudioDeviceContainer(const FAudioDeviceParams& InParams, Audio::FDeviceId InDeviceID, FAudioDeviceManager* DeviceManager)
	: NumberOfHandlesToThisDevice(0)
	, Scope(InParams.Scope)
	, bIsNonRealtime(InParams.bIsNonRealtime)
	, SpecifiedModule(InParams.AudioModule)
{
	// Here we create an entirely new audio device.
	if (bIsNonRealtime)
	{
		IAudioDeviceModule* NonRealtimeModule = FModuleManager::LoadModulePtr<IAudioDeviceModule>(TEXT("NonRealtimeAudioRenderer"));
		check(NonRealtimeModule);
		Device = NonRealtimeModule->CreateAudioDevice();
	}
	else if (SpecifiedModule != nullptr)
	{
		Device = SpecifiedModule->CreateAudioDevice();
	}
	else
	{
		check(DeviceManager->AudioDeviceModule);
		Device = DeviceManager->AudioDeviceModule->CreateAudioDevice();

		if (!Device)
		{
			Device = new Audio::FMixerDevice(DeviceManager->AudioDeviceModule->CreateAudioMixerPlatformInterface());
		}
	}

	check(Device);

	// Set to highest max channels initially provided by any quality setting, so that
	// setting to lower quality but potentially returning to higher quality later at
	// runtime is supported.
	const int32 HighestMaxChannels = GetDefault<UAudioSettings>()->GetHighestMaxChannels();
	if (Device->Init(InDeviceID, HighestMaxChannels, InParams.BufferSizeOverride, InParams.NumBuffersOverride))
	{
		const FAudioQualitySettings& QualitySettings = Device->GetQualityLevelSettings();
		Device->SetMaxChannels(QualitySettings.MaxChannels);
		Device->FadeIn();
	}
	else
	{
		UE_LOGF(LogAudio, Warning, "FAudioDevice::Init Failed!");
		Device->Teardown();
		delete Device;
		Device = nullptr;
	}
}

FAudioDeviceManager::FAudioDeviceContainer::FAudioDeviceContainer()
{
	checkNoEntry();
}

FAudioDeviceManager::FAudioDeviceContainer::FAudioDeviceContainer(FAudioDeviceContainer&& Other)
{
	Device = Other.Device;
	Other.Device = nullptr;

	NumberOfHandlesToThisDevice = Other.NumberOfHandlesToThisDevice;
	Other.NumberOfHandlesToThisDevice = 0;

	WorldsUsingThisDevice = MoveTemp(Other.WorldsUsingThisDevice);

	Scope = Other.Scope;
	Other.Scope = EAudioDeviceScope::Default;

	bIsNonRealtime = Other.bIsNonRealtime;
	Other.bIsNonRealtime = false;

	SpecifiedModule = Other.SpecifiedModule;
	Other.SpecifiedModule = nullptr;

#if INSTRUMENT_AUDIODEVICE_HANDLES
	HandleCreationStackWalks = MoveTemp(Other.HandleCreationStackWalks);
#endif
}

FAudioDeviceManager::FAudioDeviceContainer::~FAudioDeviceContainer()
{
	// Shutdown the audio device.
	if (NumberOfHandlesToThisDevice != 0)
	{
		UE_LOGF(LogAudio, Display, "Shutting down audio device while %d references to it are still alive. For more information, compile with INSTRUMENT_AUDIODEVICE_HANDLES.", NumberOfHandlesToThisDevice);

#if INSTRUMENT_AUDIODEVICE_HANDLES
		FString ActiveDeviceHandles;
		for (auto& StackWalkString : HandleCreationStackWalks)
		{
			ActiveDeviceHandles += StackWalkString.Value;
			ActiveDeviceHandles += TEXT("\n\n");
		}

		UE_LOGF(LogAudio, Warning, "List Of Active Handles: \n%ls", *ActiveDeviceHandles);
#endif
	}

	if (Device)
	{
		Device->FadeOut();
		Device->Teardown();
		delete Device;
		Device = nullptr;
	}
}

FAudioDeviceManagerDelegates::FOnAudioDeviceCreated FAudioDeviceManagerDelegates::OnAudioDeviceCreated;
FAudioDeviceManagerDelegates::FOnPreAudioDeviceDestroyed FAudioDeviceManagerDelegates::OnPreAudioDeviceDestroyed;
FAudioDeviceManagerDelegates::FOnAudioDeviceDestroyed FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed;
