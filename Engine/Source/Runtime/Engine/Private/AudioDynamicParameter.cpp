// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioDynamicParameter.h"

FDynamicParameter::FDynamicParameter(float Value)
	: CurrValue(Value)
	, StartValue(Value)
	, DeltaValue(0.0f)
	, CurrTimeSec(0.0f)
	, DurationSec(0.0f)
	, LastTime(0.0f)
	, TargetValue(Value)
{}

void FDynamicParameter::Set(float Value, float InDuration)
{
	if (TargetValue != Value || DurationSec != InDuration)
	{
		TargetValue = Value;
		if (InDuration > 0.0f)
		{
			DeltaValue = Value - CurrValue;
			StartValue = CurrValue;
			DurationSec = InDuration;
			CurrTimeSec = 0.0f;
		}
		else
		{
			StartValue = Value;
			DeltaValue = 0.0f;
			DurationSec = 0.0f;
			CurrValue = Value;
		}
	}
}

void FDynamicParameter::Update(float DeltaTime)
{
	if (DurationSec > 0.0f)
	{
		float TimeFraction = CurrTimeSec / DurationSec;
		if (TimeFraction < 1.0f)
		{
			CurrValue = DeltaValue * TimeFraction + StartValue;
		}
		else
		{
			CurrValue = StartValue + DeltaValue;
			DurationSec = 0.0f;
		}
		CurrTimeSec += DeltaTime;
	}
}