// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AudioEffect.cpp: Unreal base audio.
=============================================================================*/

#include "AudioEffect.h"
#include "Engine/Engine.h"
#include "Misc/App.h"
#include "Sound/ReverbEffect.h"
#include "AudioDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioEffect)

#define ENABLE_REVERB_SETTINGS_PRINTING 0

/** 
 * Default settings for a null reverb effect
 */
FAudioReverbEffect::FAudioReverbEffect()
{
	Time = 0.0;
	Volume = 0.0f;

	Density = 1.0f;
	Diffusion = 1.0f;
	Gain = 0.32f;
	GainHF = 0.89f;
	DecayTime = 1.49f;
	DecayHFRatio = 0.83f;
	ReflectionsGain = 0.05f;
	ReflectionsDelay = 0.007f;
	LateGain = 1.26f;
	LateDelay = 0.011f;
	AirAbsorptionGainHF = 0.994f;
	RoomRolloffFactor = 0.0f;

	bBypassEarlyReflections = true;
	bBypassLateReflections = false;
}

/** 
 * Construct generic reverb settings based in the I3DL2 standards
 */
FAudioReverbEffect::FAudioReverbEffect(
	float InRoom,
	float InRoomHF,
	float InRoomRolloffFactor,
	float InDecayTime,
	float InDecayHFRatio,
	float InReflections,
	float InReflectionsDelay,
	float InReverb,
	float InReverbDelay,
	float InDiffusion,
	float InDensity,
	float InAirAbsorption,
	bool bInBypassEarlyReflections,
	bool bInBypassLateReflections
)
{
	Time = 0.0;
	Volume = 0.0f;

	Density = InDensity;
	Diffusion = InDiffusion;
	Gain = InRoom;
	GainHF = InRoomHF;
	DecayTime = InDecayTime;
	DecayHFRatio = InDecayHFRatio;
	ReflectionsGain = InReflections;
	ReflectionsDelay = InReflectionsDelay;
	LateGain = InReverb;
	LateDelay = InReverbDelay;
	RoomRolloffFactor = InRoomRolloffFactor;
	AirAbsorptionGainHF = InAirAbsorption;

	bBypassEarlyReflections = bInBypassEarlyReflections;
	bBypassLateReflections = bInBypassLateReflections;
}

FAudioReverbEffect& FAudioReverbEffect::operator=(class UReverbEffect* InReverbEffect)
{
	if( InReverbEffect )
	{
		Density = InReverbEffect->Density;
		Diffusion = InReverbEffect->Diffusion;
		Gain = InReverbEffect->Gain;
		GainHF = InReverbEffect->GainHF;
		DecayTime = InReverbEffect->DecayTime;
		DecayHFRatio = InReverbEffect->DecayHFRatio;
		ReflectionsGain = InReverbEffect->ReflectionsGain;
		ReflectionsDelay = InReverbEffect->ReflectionsDelay;
		LateGain = InReverbEffect->LateGain;
		LateDelay = InReverbEffect->LateDelay;
		AirAbsorptionGainHF = InReverbEffect->AirAbsorptionGainHF;

		bBypassEarlyReflections = InReverbEffect->bBypassEarlyReflections;
		bBypassLateReflections = InReverbEffect->bBypassLateReflections;
	}

	return *this;
}

void FAudioReverbEffect::PrintSettings() const
{
#if ENABLE_REVERB_SETTINGS_PRINTING
	const char* FmtText =
		"\nVolume: %.4f\n"
		"Density: %.4f\n"
		"Diffusion: %.4f\n"
		"Gain: %.4f\n"
		"GainHF: %.4f\n"
		"DecayTime: %.4f\n"
		"DecayHFRatio: %.4f\n"
		"ReflectionsGain: %.4f\n"
		"ReflectionsDelay: %.4f\n"
		"LateGain: %.4f\n"
		"LateDelay: %.4f\n"
		"AirAbsorptionGainHF: %.4f\n"
		"RoomRolloffFactor: %.4f\n";

	FString FmtString(FmtText);

	FString Params = FString::Printf(
		*FmtString,
		Settings.Volume,
		Settings.Density,
		Settings.Diffusion,
		Settings.Gain,
		Settings.GainHF,
		Settings.DecayTime,
		Settings.DecayHFRatio,
		Settings.ReflectionsGain,
		Settings.ReflectionsDelay,
		Settings.LateGain,
		Settings.LateDelay,
		Settings.AirAbsorptionGainHF,
		Settings.RoomRolloffFactor
	);

	UE_LOGF(LogTemp, Log, "%ls", *Params);
#endif // ENABLE_REVERB_SETTINGS_PRINTING
}

/** 
 * Get interpolated reverb parameters
 */
bool FAudioReverbEffect::Interpolate(const FAudioEffectParameters& InStart, const FAudioEffectParameters& InEnd)
{
	const FAudioReverbEffect& Start = static_cast<const FAudioReverbEffect&>(InStart);
	const FAudioReverbEffect& End = static_cast<const FAudioReverbEffect&>(InEnd);

	float InterpValue = 1.0f;
	if (End.Time - Start.Time > 0.0)
	{
		InterpValue = (float)((FApp::GetCurrentTime() - Start.Time) / (End.Time - Start.Time));
	}

	if (InterpValue >= 1.0f)
	{
		*this = End;
		return true;
	}

	if (InterpValue <= 0.0f)
	{
		*this = Start;
		return false;
	}

	float InvInterpValue = 1.0f - InterpValue;
	Time = FApp::GetCurrentTime();

	auto InterpolateWithBypass = [&](bool bInStartBypass, float InStartValue, bool bInEndBypass, float InEndValue, float InBypassValue)
	{
		if (bInStartBypass)
		{
			InStartValue = InBypassValue;
		}
		if (bInEndBypass)
		{
			InEndValue = InBypassValue;
		}

		return (InStartValue * InvInterpValue) + (InEndValue * InterpValue);
	};

	Volume = (Start.Volume * InvInterpValue) + (End.Volume * InterpValue);

	// Early Reflections
	GainHF = InterpolateWithBypass(Start.bBypassEarlyReflections, Start.GainHF, End.bBypassEarlyReflections, End.GainHF, 0.f);
	ReflectionsGain = InterpolateWithBypass(Start.bBypassEarlyReflections, Start.ReflectionsGain, End.bBypassEarlyReflections, End.ReflectionsGain, 0.f);
	ReflectionsDelay = InterpolateWithBypass(Start.bBypassEarlyReflections, Start.ReflectionsDelay, End.bBypassEarlyReflections, End.ReflectionsDelay, 0.f);
	
	// Late Reflections
	Density = InterpolateWithBypass(Start.bBypassLateReflections, Start.Density, End.bBypassLateReflections, End.Density, 0.f);
	Diffusion = InterpolateWithBypass(Start.bBypassLateReflections, Start.Diffusion, End.bBypassLateReflections, End.Diffusion, 0.f);
	Gain = InterpolateWithBypass(Start.bBypassLateReflections, Start.Gain, End.bBypassLateReflections, End.Gain, 0.f);
	DecayTime = InterpolateWithBypass(Start.bBypassLateReflections, Start.DecayTime, End.bBypassLateReflections, End.DecayTime, 0.f);
	DecayHFRatio = InterpolateWithBypass(Start.bBypassLateReflections, Start.DecayHFRatio, End.bBypassLateReflections, End.DecayHFRatio, 0.f);
	LateGain = InterpolateWithBypass(Start.bBypassLateReflections, Start.LateGain, End.bBypassLateReflections, End.LateGain, 0.f);
	LateDelay = InterpolateWithBypass(Start.bBypassLateReflections, Start.LateDelay, End.bBypassLateReflections, End.LateDelay, 0.f);
	AirAbsorptionGainHF = InterpolateWithBypass(Start.bBypassLateReflections, Start.AirAbsorptionGainHF, End.bBypassLateReflections, End.AirAbsorptionGainHF, 0.f);
	RoomRolloffFactor = InterpolateWithBypass(Start.bBypassLateReflections, Start.RoomRolloffFactor, End.bBypassLateReflections, End.RoomRolloffFactor, 0.f);

	// Bypass only if both are bypassed. 
	bBypassEarlyReflections = Start.bBypassEarlyReflections && End.bBypassEarlyReflections;
	bBypassLateReflections = Start.bBypassLateReflections && End.bBypassLateReflections;

	return false;
}

/** 
 * Validate all settings are in range
 */
void FAudioEQEffect::ClampValues( void )
{
	FrequencyCenter0 = FMath::Clamp(FrequencyCenter0, MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY);
	FrequencyCenter1 = FMath::Clamp(FrequencyCenter1, MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY);
	FrequencyCenter2 = FMath::Clamp(FrequencyCenter2, MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY);
	FrequencyCenter3 = FMath::Clamp(FrequencyCenter3, MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY);

	Gain0 = FMath::Clamp(Gain0, 0.0f, MAX_FILTER_GAIN);
	Gain1 = FMath::Clamp(Gain1, 0.0f, MAX_FILTER_GAIN);
	Gain2 = FMath::Clamp(Gain2, 0.0f, MAX_FILTER_GAIN);
	Gain3 = FMath::Clamp(Gain3, 0.0f, MAX_FILTER_GAIN);

	Bandwidth0 = FMath::Clamp(Bandwidth0, MIN_FILTER_BANDWIDTH, MAX_FILTER_BANDWIDTH);
	Bandwidth1 = FMath::Clamp(Bandwidth1, MIN_FILTER_BANDWIDTH, MAX_FILTER_BANDWIDTH);
	Bandwidth2 = FMath::Clamp(Bandwidth2, MIN_FILTER_BANDWIDTH, MAX_FILTER_BANDWIDTH);
	Bandwidth3 = FMath::Clamp(Bandwidth3, MIN_FILTER_BANDWIDTH, MAX_FILTER_BANDWIDTH);

}

#define ENABLE_EQ_SETTINGS_PRINTING 0

void FAudioEQEffect::PrintSettings() const
{
#if ENABLE_EQ_SETTINGS_PRINTING
	const char* FmtText =
		"\nFrequencyCenter0: %.4f\n"
		"Gain0: %.4f\n"
		"Bandwidth0: %.4f\n"
		"FrequencyCenter1: %.4f\n"
		"Gain1: %.4f\n"
		"Bandwidth1: %.4f\n"
		"FrequencyCenter2: %.4f\n"
		"Gain2: %.4f\n"
		"Bandwidth2: %.4f\n"
		"FrequencyCenter3: %.4f\n"
		"Gain3: %.4f\n"
		"Bandwidth3: %.4f\n";

	FString FmtString(FmtText);

	FString Params = FString::Printf(
		*FmtString,
		Settings.FrequencyCenter0,
		Settings.Gain0,
		Settings.Bandwidth0,
		Settings.FrequencyCenter1,
		Settings.Gain1,
		Settings.Bandwidth1,
		Settings.FrequencyCenter2,
		Settings.Gain2,
		Settings.Bandwidth2,
		Settings.FrequencyCenter3,
		Settings.Gain3,
		Settings.Bandwidth3
	);

	UE_LOGF(LogTemp, Log, "%ls", *Params);
#endif
}

/** 
 * Interpolate EQ settings based on time
 */
bool FAudioEQEffect::Interpolate(const FAudioEffectParameters& InStart, const FAudioEffectParameters& InEnd)
{
	const FAudioEQEffect& Start = static_cast<const FAudioEQEffect&>(InStart);
	const FAudioEQEffect& End = static_cast<const FAudioEQEffect&>(InEnd);

	float InterpValue = 1.0f;
	if (End.RootTime - Start.RootTime > 0.0)
	{
		InterpValue = (float)((FApp::GetCurrentTime() - Start.RootTime) / (End.RootTime - Start.RootTime));
	}

	if (InterpValue >= 1.0f)
	{
		*this = End;
		return true;
	}

	if (InterpValue <= 0.0f)
	{
		*this = Start;
		return false;
	}

	RootTime = FApp::GetCurrentTime();

	FrequencyCenter0 = FMath::Lerp(Start.FrequencyCenter0, End.FrequencyCenter0, InterpValue);
	FrequencyCenter1 = FMath::Lerp(Start.FrequencyCenter1, End.FrequencyCenter1, InterpValue);
	FrequencyCenter2 = FMath::Lerp(Start.FrequencyCenter2, End.FrequencyCenter2, InterpValue);
	FrequencyCenter3 = FMath::Lerp(Start.FrequencyCenter3, End.FrequencyCenter3, InterpValue);

	Gain0 = FMath::Lerp(Start.Gain0, End.Gain0, InterpValue);
	Gain1 = FMath::Lerp(Start.Gain1, End.Gain1, InterpValue);
	Gain2 = FMath::Lerp(Start.Gain2, End.Gain2, InterpValue);
	Gain3 = FMath::Lerp(Start.Gain3, End.Gain3, InterpValue);

	Bandwidth0 = FMath::Lerp(Start.Bandwidth0, End.Bandwidth0, InterpValue);
	Bandwidth1 = FMath::Lerp(Start.Bandwidth1, End.Bandwidth1, InterpValue);
	Bandwidth2 = FMath::Lerp(Start.Bandwidth2, End.Bandwidth2, InterpValue);
	Bandwidth3 = FMath::Lerp(Start.Bandwidth3, End.Bandwidth3, InterpValue);

	return false;
}

/**
 * Converts and volume (0.0f to 1.0f) to a deciBel value
 */
int64 FAudioEffectsManager::VolumeToDeciBels( float Volume )
{
	int64 DeciBels = -100;

	if( Volume > 0.0f )
	{
		DeciBels = FMath::Clamp<int64>( ( int64 )( 20.0f * log10f( Volume ) ), -100, 0 ) ;
	}

	return( DeciBels );
}


/**
 * Converts and volume (0.0f to 1.0f) to a MilliBel value (a Hundredth of a deciBel)
 */
int64 FAudioEffectsManager::VolumeToMilliBels( float Volume, int32 MaxMilliBels )
{
	int64 MilliBels = -10000;

	if( Volume > 0.0f )
	{
		MilliBels = FMath::Clamp<int64>( ( int64)( 2000.0f * log10f( Volume ) ), -10000, MaxMilliBels );
	}

	return( MilliBels );
}

/** 
 * Clear out any reverb and EQ settings
 */
FAudioEffectsManager::FAudioEffectsManager( FAudioDevice* InDevice )
	: AudioDevice(InDevice)
	, bEffectsInitialised(false)
	, CurrentReverbAsset(nullptr)
	, CurrentEQMix(nullptr)
	, bReverbActive(false)
	, bEQActive(false)
	, bReverbChanged(false)
	, bEQChanged(false)
{
	InitAudioEffects();
}

void FAudioEffectsManager::ResetInterpolation()
{
	InitAudioEffects();
}

/** 
 * Sets up default reverb and eq settings
 */
void FAudioEffectsManager::InitAudioEffects( void )
{
	FMemory::Memzero(&PrevReverbEffect, sizeof(PrevReverbEffect));

	ClearMixSettings();
}

void FAudioEffectsManager::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(CurrentReverbAsset);
}

/**
 * Called every tick from UGameViewportClient::Draw
 * 
 * Sets new reverb mode if necessary. Otherwise interpolates to the current settings and calls SetEffect to handle
 * the platform specific aspect.
 */
void FAudioEffectsManager::SetReverbSettings( const FReverbSettings& ReverbSettings, bool bForce)
{
	/** Update the settings if the reverb has changed */
	const bool bSetNewReverb = (ReverbSettings.bApplyReverb != CurrentReverbSettings.bApplyReverb) || 
		(ReverbSettings.ReverbEffect != CurrentReverbAsset) || 
		bForce || 
		!FMath::IsNearlyEqual(ReverbSettings.Volume, CurrentReverbSettings.Volume);

	if(bSetNewReverb)
	{
		FString CurrentReverbName = CurrentReverbAsset ? CurrentReverbAsset->GetName() : TEXT("None");
		FString NextReverbName = ReverbSettings.ReverbEffect ? ReverbSettings.ReverbEffect->GetName() : TEXT("None");
		UE_LOGF(LogAudio, Log, "FAudioDevice::SetReverbSettings(): Old - %ls  New - %ls:%f (%f)",
			*CurrentReverbName, *NextReverbName, ReverbSettings.Volume, ReverbSettings.FadeTime );

		if( ReverbSettings.Volume > 1.0f )
		{
			UE_LOGF(LogAudio, Warning, "FAudioDevice::SetReverbSettings(): Illegal volume %g (should be 0.0f <= Volume <= 1.0f)", ReverbSettings.Volume );
		}

		CurrentReverbSettings = ReverbSettings;

		SourceReverbEffect = CurrentReverbEffect;
		SourceReverbEffect.Time = FApp::GetCurrentTime();

		DestinationReverbEffect = ReverbSettings.ReverbEffect;
		bReverbChanged = true;

		if (bForce)
		{
			DestinationReverbEffect.Time = FApp::GetCurrentTime();
		}
		else
		{
			DestinationReverbEffect.Time = FApp::GetCurrentTime() + ReverbSettings.FadeTime;
		}

		DestinationReverbEffect.Volume = ReverbSettings.Volume;

		if((nullptr == ReverbSettings.ReverbEffect) || !ReverbSettings.bApplyReverb)
		{
			DestinationReverbEffect.Volume = 0.0f;
		}

		CurrentReverbAsset = ReverbSettings.ReverbEffect;
	}
}

/**
 * Sets new EQ mix if necessary. Otherwise interpolates to the current settings and calls SetEffect to handle
 * the platform specific aspect.
 */
void FAudioEffectsManager::SetMixSettings(USoundMix* NewMix, bool bIgnorePriority, bool bForce)
{
	if (NewMix && (NewMix != CurrentEQMix || bForce))
	{
		// Check whether the priority of this SoundMix is higher than existing one
		if (CurrentEQMix == NULL || bIgnorePriority || NewMix->EQPriority > CurrentEQMix->EQPriority)
		{
			UE_LOGF(LogAudio, Log, "FAudioEffectsManager::SetMixSettings(): %ls", *NewMix->GetName() );

			SourceEQEffect = CurrentEQEffect;
			SourceEQEffect.RootTime = FApp::GetCurrentTime();

			if( NewMix->bApplyEQ )
			{
				DestinationEQEffect = NewMix->EQSettings;
			}
			else
			{
				// it doesn't have EQ settings, so interpolate back to default
				DestinationEQEffect = FAudioEQEffect();
			}

			DestinationEQEffect.RootTime = FApp::GetCurrentTime() + NewMix->FadeInTime;
			DestinationEQEffect.ClampValues();

			bEQChanged = true;

			CurrentEQMix = NewMix;
		}
	}
}

/**
 * If there is an active SoundMix, clear it and any EQ settings it applied
 */
void FAudioEffectsManager::ClearMixSettings()
{
	if (CurrentEQMix)
	{
		UE_LOGF(LogAudio, Log, "FAudioEffectsManager::ClearMixSettings(): %ls", *CurrentEQMix->GetName() );

		SourceEQEffect = CurrentEQEffect;
		double CurrentTime = FApp::GetCurrentTime();
		SourceEQEffect.RootTime = CurrentTime;

		// interpolate back to default
		DestinationEQEffect = FAudioEQEffect();
		DestinationEQEffect.RootTime = CurrentTime + CurrentEQMix->FadeOutTime;

		CurrentEQMix = NULL;
	}
}


static bool operator==(const FAudioReverbEffect& EffectA, const FAudioReverbEffect& EffectB)
{
	bool bIsEqual = true;

	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.Volume, EffectB.Volume, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.Density, EffectB.Density, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.Diffusion, EffectB.Diffusion, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.Gain, EffectB.Gain, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.GainHF, EffectB.GainHF, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.DecayTime, EffectB.DecayTime, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.DecayHFRatio, EffectB.DecayHFRatio, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.ReflectionsGain, EffectB.ReflectionsGain, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.ReflectionsDelay, EffectB.ReflectionsDelay, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.LateGain, EffectB.LateGain, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.LateDelay, EffectB.LateDelay, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.AirAbsorptionGainHF, EffectB.AirAbsorptionGainHF, UE_KINDA_SMALL_NUMBER);
	bIsEqual = bIsEqual && FMath::IsNearlyEqual(EffectA.RoomRolloffFactor, EffectB.RoomRolloffFactor, UE_KINDA_SMALL_NUMBER);

	return bIsEqual;
}

static bool operator!=(const FAudioReverbEffect& EffectA, const FAudioReverbEffect& EffectB)
{
	return !(EffectA == EffectB);
}

/** 
 * Feed in new settings to the audio effect system
 */
void FAudioEffectsManager::Update()
{
	// Check for changes to the mix so we can hear EQ changes in real-time
#if WITH_EDITORONLY_DATA
	if (CurrentEQMix && CurrentEQMix->bChanged)
	{
		CurrentEQMix->bChanged = false;
		SetMixSettings(CurrentEQMix, true, true);
	}

	if (CurrentReverbAsset && CurrentReverbAsset->bChanged)
	{
		CurrentReverbAsset->bChanged = false;
		class FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		TArray<FAudioDevice*> AudioDevices = DeviceManager->GetAudioDevices();
		for (int32 i = 0; i < AudioDevices.Num(); ++i)
		{
			if (AudioDevices[i])
			{
				FAudioEffectsManager* EffectsManager = AudioDevices[i]->GetEffects();
				EffectsManager->SetReverbSettings(CurrentReverbSettings, true);
			}
		}
	}

#endif

	const bool bIsReverbDone = CurrentReverbEffect.Interpolate(SourceReverbEffect, DestinationReverbEffect);
	if (!bIsReverbDone || bReverbActive || bReverbChanged)
	{
		bReverbChanged = false;
		PrevReverbEffect = CurrentReverbEffect;
		bReverbActive = !bIsReverbDone;
		SetReverbEffectParameters(CurrentReverbEffect);
	}

	const bool bIsEQDone = CurrentEQEffect.Interpolate(SourceEQEffect, DestinationEQEffect);
	if (!bIsEQDone || bEQActive || bEQChanged)
	{
		bEQChanged = false;
		bEQActive = !bIsEQDone;
		SetEQEffectParameters(CurrentEQEffect);
	}
}

// end 

