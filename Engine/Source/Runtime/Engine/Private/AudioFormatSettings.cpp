// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/AudioFormatSettings.h"
#include "Features/IModularFeatures.h"
#include "Sound/SoundWave.h"
#include "Misc/ConfigCacheIni.h"
#include "Audio.h"
#include "AudioCompressionSettingsUtils.h"
#include "ISoundWaveCloudStreaming.h"
#include "Interfaces/IAudioFormat.h"

namespace Audio
{
	static bool ShouldAllowHardwareFormats()
	{
		/** AudioLink allows other audio engines to take control of the hardware
		 *  That prevents us creating hardware codecs in most cases so disable them here */
		static bool IsAudioLinkEnabled = []() -> bool
		{
			const bool bAvailable = IModularFeatures::Get().IsModularFeatureAvailable(TEXT("AudioLink Factory"));
			UE_CLOGF(bAvailable,LogAudio, Display, "AudioLink is enabled, disabling hardware AudioFormats.");
			return bAvailable;
		}();
		return !IsAudioLinkEnabled;
	}

	FName GetCloudStreamingFormatOverride(const FName& InCurrentFormat, const USoundWave* InWave)
	{
#if WITH_EDITORONLY_DATA
		// Is a cloud streaming feature available?
		if (InWave->IsCloudStreamingEnabled())
		{
			IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
			TArray<Audio::ISoundWaveCloudStreamingFeature*> Features = IModularFeatures::Get().GetModularFeatureImplementations<Audio::ISoundWaveCloudStreamingFeature>(Audio::ISoundWaveCloudStreamingFeature::GetModularFeatureName());
			// If there is more than one cloud streaming feature it will be ambiguous which one to use.
			check(Features.Num() <= 1);
			for(int32 i=0; i<Features.Num(); ++i)
			{
				if (Features[i]->CanOverrideFormat(InWave))
				{
					FName NewFormatName = Features[i]->GetOverrideFormatName(InWave);
					if (NewFormatName.GetStringLength())
					{
						return NewFormatName;
					}
				}
			}
		}
#endif // WITH_EDITORONLY_DATA
		return InCurrentFormat;
	}
	
	FAudioFormatSettings::FAudioFormatSettings(FConfigCacheIni* InConfigSystem, const FString& InConfigFile, const FString& InIniPlatformName)
		: IniPlatformName(*InIniPlatformName)
	{
		ReadConfiguration(InConfigSystem, InConfigFile); 
	}

	// Few things that the platform might define about a wave.
	struct FAudioFormatSettings::FPlatformWaveState
	{
		FName FormatName;
		FName Name;
		int32 SampleRate = 0;
		int32 NumChannels = 0;

		// From a SoundWave.
		FPlatformWaveState(const USoundWave* InWave,const FAudioFormatSettings* InFormatSettings)
			: FormatName(ToName(InWave->GetSoundAssetCompressionType()))
			, Name(InWave->GetName())
			, SampleRate(InWave->GetImportedSampleRate())
			, NumChannels(InWave->NumChannels)
		{
			// Override sample-rate? (only if resample for device is true).
			if (const FPlatformAudioCookOverrides* CookOverrides = FPlatformCompressionUtilities::GetCookOverrides(*InFormatSettings->IniPlatformName.ToString()) )
			{
				if (const float SampleRateOverride = InWave->GetSampleRateForCompressionOverrides(CookOverrides); CookOverrides->bResampleForDevice && SampleRateOverride != -1.f)
				{
					SampleRate = SampleRateOverride;
				}
			}

			// Platform Specific? (resolve to this platforms format)
			if (FormatName == NAME_PLATFORM_SPECIFIC)
			{
				// Convert "PlatformSpecific" into format name, based on our config for this platform.
				FormatName = InWave->IsStreaming() ?
					InFormatSettings->PlatformStreamingFormat :
					InFormatSettings->PlatformFormat;
			}
		}
	};	

	const IAudioFormat* FAudioFormatSettings::FindFormat(const FName& InFormatName) const
	{
		// In cache?
		FScopeLock Lock(&AudioFormatCacheCs);
		if (const IAudioFormat* const* Found = AudioFormatCache.Find(InFormatName))
		{
			return *Found;
		}
		
		// Look it up and cache.
		const TArray<IAudioFormat*> AllFormats = IModularFeatures::Get().GetModularFeatureImplementations<IAudioFormat>(IAudioFormat::GetModularFeatureName());
		if (IAudioFormat* const * Found = AllFormats.FindByPredicate
			([InFormatName](const IAudioFormat* InFormat) -> bool
			{
				TArray<FName> SupportedFormats;
				InFormat->GetSupportedFormats(SupportedFormats);
				return SupportedFormats.Contains(InFormatName);
			}))
		{
			AudioFormatCache.Add(InFormatName, *Found);
			return *Found;
		}

		// Fail.
		return nullptr;	
	}

	bool FAudioFormatSettings::IsFormatAllowed(const FPlatformWaveState& InWave) const 
	{		
		if (const IAudioFormat* Format = FindFormat(InWave.FormatName))
		{
			// Platform supported?
			if (!Format->IsPlatformSupported(IniPlatformName))
			{
				UE_LOGF(LogAudio, Verbose, "Wave '%ls', format '%ls' doesn't support platform '%ls'",
						*InWave.Name.ToString(), *InWave.FormatName.ToString(), *IniPlatformName.ToString());
				return false;		
			}
			
		
			// Channel count ok?
			if (!Format->IsChannelCountSupported(InWave.NumChannels))
			{
				UE_LOGF(LogAudio, Verbose, "Wave '%ls', format '%ls' doesn't support channel count: '%d'",
					*InWave.Name.ToString(), *InWave.FormatName.ToString(), InWave.NumChannels);
				return false;	
			}

			// Hardware ok?
			if (Format->IsHardwareFormat() && !ShouldAllowHardwareFormats())
			{
				return false;
			}

			// Success, passed all tests.
			return true;
		}

		// Assume no, if we can't find it registered.
		return false;
	}

	FName FAudioFormatSettings::GetWaveFormat(const USoundWave* Wave) const
	{
		FPlatformWaveState PlatformWave(Wave, this);
		
		// Can we use the one that's defined based on its constraints?
		if (!IsFormatAllowed(PlatformWave))
		{
			PlatformWave.FormatName = FallbackFormat;
		}
		
		PlatformWave.FormatName = GetCloudStreamingFormatOverride(PlatformWave.FormatName, Wave);
		return PlatformWave.FormatName;
	}

	void FAudioFormatSettings::GetAllWaveFormats(TArray<FName>& OutFormats) const
	{
		OutFormats = AllWaveFormats;
	}

	void FAudioFormatSettings::GetWaveFormatModuleHints(TArray<FName>& OutHints) const
	{
		OutHints = WaveFormatModuleHints;
	}

	void FAudioFormatSettings::ReadConfiguration(FConfigCacheIni* InConfigSystem, const FString& InConfigFilename)
	{
		auto MakePrettyArrayToString = [](const TArray<FName>& InNames) -> FString 
		{
			return FString::JoinBy(InNames, TEXT(", "),[](const FName& i) -> FString { return i.GetPlainNameString(); });
		};
		
		auto ToFName = [](const FString& InName) -> FName { return { *InName }; };
			
		using namespace Audio;

		// AllWaveFormats.
		{		
			TArray<FString> FormatNames;
			if (InConfigSystem->GetArray(TEXT("Audio"), TEXT("AllWaveFormats"), FormatNames, InConfigFilename))
			{
				Algo::Transform(FormatNames, AllWaveFormats, ToFName);
			}
			else
			{
				AllWaveFormats = { NAME_BINKA, NAME_ADPCM, NAME_PCM, NAME_OPUS, NAME_RADA};
				UE_LOGF(LogAudio, Warning, "Audio:AllWaveFormats is not defined, defaulting to built in formats. (%ls)", *MakePrettyArrayToString(AllWaveFormats));
			}
		}

		// FormatModuleHints
		{		
			TArray<FString> FormatModuleHints;
			if (InConfigSystem->GetArray(TEXT("Audio"), TEXT("FormatModuleHints"), FormatModuleHints, InConfigFilename))
			{
				Algo::Transform(FormatModuleHints, WaveFormatModuleHints, ToFName);
			}
		}

		// FallbackFormat
		{
			FString FallbackFormatString;
			if (InConfigSystem->GetString(TEXT("Audio"), TEXT("FallbackFormat"), FallbackFormatString, InConfigFilename))
			{
				FallbackFormat = *FallbackFormatString;
			}
			else
			{
				FallbackFormat = NAME_ADPCM;
				UE_LOGF(LogAudio, Warning, "Audio:FallbackFormat is not defined, defaulting to '%ls'.", *FallbackFormat.GetPlainNameString());
			}
			if (!AllWaveFormats.Contains(FallbackFormat) && AllWaveFormats.Num() > 0)
			{
				UE_LOGF(LogAudio, Warning, "FallbackFormat '%ls' not defined in 'AllWaveFormats'. Using first format listed '%ls'", *FallbackFormatString, *AllWaveFormats[0].ToString());
				FallbackFormat = AllWaveFormats[0];
			}
		}

		// PlatformFormat
		{
			FString PlatformFormatString;
			if (InConfigSystem->GetString(TEXT("Audio"), TEXT("PlatformFormat"), PlatformFormatString, InConfigFilename))
			{
				PlatformFormat = *PlatformFormatString;
			}
			else
			{
				PlatformFormat = NAME_ADPCM;
				UE_LOGF(LogAudio, Warning, "Audio:PlatformFormat is not defined, defaulting to '%ls'.", *PlatformFormat.GetPlainNameString());
			}
			if (!AllWaveFormats.Contains(PlatformFormat))
			{
				UE_LOGF(LogAudio, Warning, "PlatformFormat '%ls' not defined in 'AllWaveFormats'. Using fallback format '%ls'", *PlatformFormatString, *FallbackFormat.ToString());
				PlatformFormat = FallbackFormat;
			}
		}

		// PlatformStreamingFormat
		{
			FString PlatformStreamingFormatString;
			if (InConfigSystem->GetString(TEXT("Audio"), TEXT("PlatformStreamingFormat"), PlatformStreamingFormatString, InConfigFilename))
			{
				PlatformStreamingFormat = *PlatformStreamingFormatString;		
			}
			else
			{
				PlatformStreamingFormat = NAME_ADPCM;
				UE_LOGF(LogAudio, Warning, "Audio:PlatformStreamingFormat is not defined, defaulting to '%ls'.", *PlatformStreamingFormat.GetPlainNameString());
			}
			if (!AllWaveFormats.Contains(PlatformStreamingFormat))
			{
				UE_LOGF(LogAudio, Warning, "PlatformStreamingFormat '%ls' not defined in 'AllWaveFormats'. Using fallback format '%ls'", *PlatformStreamingFormatString, *FallbackFormat.ToString());
				PlatformStreamingFormat = FallbackFormat;
			}
		}

		UE_LOGF(LogAudio, Verbose, "AudioFormatSettings: TargetName='%ls', AllWaveFormats=(%ls), Hints=(%ls), PlatformFormat='%ls', PlatformStreamingFormat='%ls', FallbackFormat='%ls'",
			*IniPlatformName.ToString(), *MakePrettyArrayToString(AllWaveFormats), *MakePrettyArrayToString(WaveFormatModuleHints), *PlatformFormat.ToString(), *PlatformStreamingFormat.ToString(), *FallbackFormat.ToString());	
	}


}// namespace Audio
