// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPluginUtilities.h"

#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "IAudioExtensionPlugin.h"
#include "Misc/ConfigCacheIni.h"


static FString DefaultModulationPluginCVar;
FAutoConsoleVariableRef CVarActiveModulationPlugin(
	TEXT("au.DefaultModulationPlugin"),
	DefaultModulationPluginCVar,
	TEXT("Name of default modulation plugin to load and use (overridden "
	"by platform-specific implementation name in config.\n"),
	ECVF_Default);

/** Get the target setting name for each platform type. */
FORCEINLINE const TCHAR* GetPluginConfigName(EAudioPlugin PluginType)
{
	switch (PluginType)
	{
		case EAudioPlugin::SPATIALIZATION:
			return TEXT("SpatializationPlugin");

		case EAudioPlugin::REVERB:
			return TEXT("ReverbPlugin");

		case EAudioPlugin::OCCLUSION:
			return TEXT("OcclusionPlugin");

		case EAudioPlugin::MODULATION:
			return TEXT("ModulationPlugin");

		case EAudioPlugin::SOURCEDATAOVERRIDE:
			return TEXT("SourceDataOverridePlugin");
			
		default:
			checkf(false, TEXT("Undefined audio plugin type."));
			return TEXT("");
	}
}

/************************************************************************/
/* Plugin Utilities                                                     */
/************************************************************************/
FName AudioPluginUtilities::GetDesiredSpatializationPluginName()
{
	return FName(GetDesiredPluginName(EAudioPlugin::SPATIALIZATION));
}

TArray<IAudioSpatializationFactory*> AudioPluginUtilities::GetSpatialPluginArray()
{
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	return IModularFeatures::Get().GetModularFeatureImplementations<IAudioSpatializationFactory>(IAudioSpatializationFactory::GetModularFeatureName());
}

IAudioSpatializationFactory* AudioPluginUtilities::GetDesiredSpatializationPlugin()
{
	FString DesiredSpatializationPlugin = GetDesiredPluginName(EAudioPlugin::SPATIALIZATION);

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IAudioSpatializationFactory *> SpatializationPluginFactories = IModularFeatures::Get().GetModularFeatureImplementations<IAudioSpatializationFactory>(IAudioSpatializationFactory::GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	//Iterate through all of the plugins we've discovered:
	for (IAudioSpatializationFactory* PluginFactory : SpatializationPluginFactories)
	{
		//if this plugin's name matches the name found in the platform settings, use it:
		if (PluginFactory->GetDisplayName().Equals(DesiredSpatializationPlugin))
		{
			return PluginFactory;
		}
	}

	return nullptr;
}

IAudioSourceDataOverrideFactory* AudioPluginUtilities::GetDesiredSourceDataOverridePlugin()
{
	FString DesiredSourceDataOverridePlugin = GetDesiredPluginName(EAudioPlugin::SOURCEDATAOVERRIDE);

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IAudioSourceDataOverrideFactory*> SourceDataOverridePluginFactories = IModularFeatures::Get().GetModularFeatureImplementations<IAudioSourceDataOverrideFactory>(IAudioSourceDataOverrideFactory::GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	//Iterate through all of the plugins we've discovered:
	for (IAudioSourceDataOverrideFactory* PluginFactory : SourceDataOverridePluginFactories)
	{
		//if this plugin's name matches the name found in the platform settings, use it:
		if (PluginFactory->GetDisplayName().Equals(DesiredSourceDataOverridePlugin))
		{
			return PluginFactory;
		}
	}

	return nullptr;
}


IAudioReverbFactory* AudioPluginUtilities::GetDesiredReverbPlugin()
{
	//Get the name of the desired Reverb plugin:
	FString DesiredReverbPlugin = GetDesiredPluginName(EAudioPlugin::REVERB);

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IAudioReverbFactory *> ReverbPluginFactories = IModularFeatures::Get().GetModularFeatureImplementations<IAudioReverbFactory>(IAudioReverbFactory::GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	//Iterate through all of the plugins we've discovered:
	for (IAudioReverbFactory* PluginFactory : ReverbPluginFactories)
	{
		//if this plugin's name matches the name found in the platform settings, use it:
		if (PluginFactory->GetDisplayName().Equals(DesiredReverbPlugin))
		{
			return PluginFactory;
		}
	}

	return nullptr;
}

IAudioOcclusionFactory* AudioPluginUtilities::GetDesiredOcclusionPlugin()
{
	FString DesiredOcclusionPlugin = GetDesiredPluginName(EAudioPlugin::OCCLUSION);

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IAudioOcclusionFactory *> OcclusionPluginFactories = IModularFeatures::Get().GetModularFeatureImplementations<IAudioOcclusionFactory>(IAudioOcclusionFactory::GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	//Iterate through all of the plugins we've discovered:
	for (IAudioOcclusionFactory* PluginFactory : OcclusionPluginFactories)
	{
		//if this plugin's name matches the name found in the platform settings, use it:
		if (PluginFactory->GetDisplayName().Equals(DesiredOcclusionPlugin))
		{
			return PluginFactory;
		}
	}

	return nullptr;
}

IAudioModulationFactory* AudioPluginUtilities::GetDesiredModulationPlugin()
{
	const FName& PlatformPluginName = FName(*GetDesiredPluginName(EAudioPlugin::MODULATION));
	const FName& PluginName = (PlatformPluginName == NAME_None) ? GetDefaultModulationPluginName() : PlatformPluginName;
	const FName& FeatureName = IAudioModulationFactory::GetModularFeatureName();

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IAudioModulationFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<IAudioModulationFactory>(FeatureName);
	IModularFeatures::Get().UnlockModularFeatureList();

	for (IAudioModulationFactory* Factory : Factories)
	{
		//if this plugin's name matches the name found in the platform settings, use it:
		if (Factory->GetDisplayName() == PluginName)
		{
			return Factory;
		}
	}

	return nullptr;
}

FString AudioPluginUtilities::GetDesiredPluginName(EAudioPlugin PluginType)
{
	FString PluginName;
	GConfig->GetString(FPlatformProperties::GetRuntimeSettingsClassName(), GetPluginConfigName(PluginType), PluginName, GEngineIni);
	return PluginName;
}

const FName& AudioPluginUtilities::GetDefaultModulationPluginName()
{
	static FName DefaultModulationPluginName("DefaultModulationPlugin");

	if (!DefaultModulationPluginCVar.IsEmpty())
	{
		DefaultModulationPluginName = FName(*DefaultModulationPluginCVar);
	}

	return DefaultModulationPluginName;
}
