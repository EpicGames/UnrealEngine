// Copyright Epic Games, Inc. All Rights Reserved.
#include "Sound/AudioSettings.h"

#include "Sound/AudioPlatformSettings.h"
#include "AudioBusSubsystem.h"
#include "AudioDevice.h"
#include "Sound/SoundNodeQualityLevel.h"
#include "Sound/SoundSubmix.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSettings)

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioSettings"

UAudioSettings::UAudioSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SectionName = TEXT("Audio");
	AddDefaultSettings();

	bAllowPlayWhenSilent = true;
	bParameterInterfacesRegistered = false;

	GlobalMinPitchScale = 0.4F;
	GlobalMaxPitchScale = 2.0F;

	DefaultAudioCompressionType = EDefaultAudioCompressionType::BinkAudio;

	PlatformSettings.Initialize(UAudioPlatformSettings::StaticClass());
}

void UAudioSettings::AddDefaultSettings()
{
	FAudioQualitySettings DefaultSettings;
	DefaultSettings.DisplayName = LOCTEXT("DefaultSettingsName", "Default");
	QualityLevels.Add(DefaultSettings);
	bAllowPlayWhenSilent = true;
	DefaultReverbSendLevel_DEPRECATED = 0.0f;
	VoiPSampleRate = EVoiceSampleRate::Low16000Hz;
	NumStoppingSources = 8;
}

#if WITH_EDITOR
void UAudioSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	CachedSoundClass = DefaultSoundClassName;

	// Cache master submix in case user tries to set to submix that isn't a top-level submix
	CachedMasterSubmix = MasterSubmix;

	// Cache at least the first entry in case someone tries to clear the array
	CachedQualityLevels = QualityLevels;
}

void UAudioSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		bool bReconcileQualityNodes = false;
		bool bPromptRestartRequired = false;
		FName PropertyName = PropertyChangedEvent.Property->GetFName();

		LoadDefaultObjects();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UAudioSettings, MasterSubmix))
		{
			if (USoundSubmix* NewSubmix = Cast<USoundSubmix>(MasterSubmix.TryLoad()))
			{
				if (NewSubmix->ParentSubmix)
				{
					FNotificationInfo Info(LOCTEXT("AudioSettings_InvalidMasterSubmix", "'Master Submix' cannot be set to submix with parent."));
					Info.bFireAndForget = true;
					Info.ExpireDuration = 2.0f;
					Info.bUseThrobber = true;
					FSlateNotificationManager::Get().AddNotification(Info);

					MasterSubmix = CachedMasterSubmix;
				}
			}

			bPromptRestartRequired = true;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAudioSettings, BaseDefaultSubmix)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UAudioSettings, EQSubmix)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UAudioSettings, ReverbSubmix)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UAudioSettings, PlatformSettings))
		{
			bPromptRestartRequired = true;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAudioSettings, DefaultAudioCompressionType))
		{
			// loop through all USoundWaves and update the compression type w/ the new defualt
			// (if they are set to "Project Default")
			for (TObjectIterator<USoundWave> It; It; ++It)
			{
				USoundWave* SoundWave = *It;
				if(SoundWave && SoundWave->GetSoundAssetCompressionTypeEnum() == ESoundAssetCompressionType::ProjectDefined)
				{
					// this will query the correct compression type and update the asset accordingly
					SoundWave->SetSoundAssetCompressionType(SoundWave->GetSoundAssetCompressionType(), /*bMarkDirty*/false);
				}
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAudioSettings, QualityLevels))
		{
			if (QualityLevels.Num() == 0)
			{
				QualityLevels.Add(CachedQualityLevels[0]);
			}
			else if (QualityLevels.Num() > CachedQualityLevels.Num())
			{
				for (FAudioQualitySettings& AQSettings : QualityLevels)
				{
					if (AQSettings.DisplayName.IsEmpty())
					{
						bool bFoundDuplicate;
						int32 NewQualityLevelIndex = 0;
						FText NewLevelName;
						do
						{
							bFoundDuplicate = false;
							NewLevelName = FText::Format(LOCTEXT("NewQualityLevelName","New Level{0}"), (NewQualityLevelIndex > 0 ? FText::FromString(FString::Printf(TEXT(" %d"),NewQualityLevelIndex)) : FText::GetEmpty()));
							for (const FAudioQualitySettings& QualityLevelSettings : QualityLevels)
							{
								if (QualityLevelSettings.DisplayName.EqualTo(NewLevelName))
								{
									bFoundDuplicate = true;
									break;
								}
							}
							NewQualityLevelIndex++;
						} while (bFoundDuplicate);
						AQSettings.DisplayName = NewLevelName;
					}
				}
			}

			bReconcileQualityNodes = true;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAudioQualitySettings, DisplayName))
		{
			bReconcileQualityNodes = true;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAudioSettings, DefaultAudioBuses)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(FDefaultAudioBusSettings, AudioBus))
		{
			if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
			{
				DeviceManager->IterateOverAllDevices([this](Audio::FDeviceId, FAudioDevice* InDevice)
				{
					UAudioBusSubsystem* AudioBusSubsystem = InDevice->GetSubsystem<UAudioBusSubsystem>();
					check(AudioBusSubsystem);
					AudioBusSubsystem->InitDefaultAudioBuses();
				});
			}
		}

		if (bReconcileQualityNodes)
		{
			for (TObjectIterator<USoundNodeQualityLevel> It; It; ++It)
			{
				It->ReconcileNode(true);
			}
		}

		if (bPromptRestartRequired)
		{
			FNotificationInfo Info(LOCTEXT("AudioSettings_ChangeRequiresEditorRestart", "Change to Audio Settings requires editor restart in order for changes to take effect."));
			Info.bFireAndForget = true;
			Info.ExpireDuration = 2.0f;
			Info.bUseThrobber = true;
			FSlateNotificationManager::Get().AddNotification(Info);
		}

		AudioSettingsChanged.Broadcast();
	}
}
#endif // WITH_EDITOR

const FAudioQualitySettings& UAudioSettings::GetQualityLevelSettings(int32 QualityLevel) const
{
	check(QualityLevels.Num() > 0);
	return QualityLevels[FMath::Clamp(QualityLevel, 0, QualityLevels.Num() - 1)];
}

int32 UAudioSettings::GetDefaultCompressionQuality() const
{
	return FMath::Clamp(DefaultCompressionQuality_DEPRECATED,1,100);
}

int32 UAudioSettings::GetQualityLevelSettingsNum() const
{
	return QualityLevels.Num();
}

void UAudioSettings::LoadDefaultObjects()
{
	UE_LOGF(LogAudio, Display, "Loading Default Audio Settings Objects...");

	// TODO: Move all soft object paths to load here (SoundMixes, Submixes, etc.)
	static const FString EngineSoundsDir = "/Engine/EngineSounds";

	if (DefaultSoundClass)
	{
		DefaultSoundClass->RemoveFromRoot();
		DefaultSoundClass = nullptr;
	}

	if (UObject* SoundClassObject = DefaultSoundClassName.TryLoad())
	{
		DefaultSoundClass = CastChecked<USoundClass>(SoundClassObject);
		DefaultSoundClass->AddToRoot();
	}

#if WITH_EDITOR
	if (!DefaultSoundClass)
	{
		DefaultSoundClassName = CachedSoundClass;
		if (UObject* SoundClassObject = DefaultSoundClassName.TryLoad())
		{
			DefaultSoundClass = CastChecked<USoundClass>(SoundClassObject);
			DefaultSoundClass->AddToRoot();
		}
	}
#endif // WITH_EDITOR

	if (!DefaultSoundClass)
	{
		UE_LOGF(LogAudio, Warning, "Failed to load Default SoundClassObject from path '%ls'.  Attempting to fall back to engine default.", *DefaultSoundClassName.GetAssetPathString());
		DefaultSoundClassName.SetPath(EngineSoundsDir / TEXT("Master"));
		if (UObject* SoundClassObject = DefaultSoundClassName.TryLoad())
		{
			DefaultSoundClass = CastChecked<USoundClass>(SoundClassObject);
			DefaultSoundClass->AddToRoot();
		}
	}

	if (!DefaultSoundClass)
	{
		UE_LOGF(LogAudio, Error, "Failed to load Default SoundClassObject from path '%ls'.", *DefaultSoundClassName.GetAssetPathString());
	}

	if (DefaultMediaSoundClass)
	{
		DefaultMediaSoundClass->RemoveFromRoot();
		DefaultMediaSoundClass = nullptr;
	}

	if (UObject* MediaSoundClassObject = DefaultMediaSoundClassName.TryLoad())
	{
		DefaultMediaSoundClass = CastChecked<USoundClass>(MediaSoundClassObject);
		DefaultMediaSoundClass->AddToRoot();
	}
	else
	{
		UE_LOGF(LogAudio, Display, "No default MediaSoundClassObject specified (or failed to load).");
	}

	if (DefaultSoundConcurrency)
	{
		DefaultSoundConcurrency->RemoveFromRoot();
		DefaultSoundConcurrency = nullptr;
	}

	if (UObject* SoundConcurrencyObject = DefaultSoundConcurrencyName.TryLoad())
	{
		DefaultSoundConcurrency = CastChecked<USoundConcurrency>(SoundConcurrencyObject);
		DefaultSoundConcurrency->AddToRoot();
	}
	else
	{
		UE_LOGF(LogAudio, Display, "No default SoundConcurrencyObject specified (or failed to load).");
	}
}

void UAudioSettings::RegisterParameterInterfaces()
{
	if (!bParameterInterfacesRegistered)
	{
		UE_LOGF(LogAudio, Display, "Registering Engine Module Parameter Interfaces...");
		bParameterInterfacesRegistered = true;
		Audio::IAudioParameterInterfaceRegistry& InterfaceRegistry = Audio::IAudioParameterInterfaceRegistry::Get();
		InterfaceRegistry.RegisterInterface(Audio::AttenuationInterface::GetInterface());
		InterfaceRegistry.RegisterInterface(Audio::SpatializationInterface::GetInterface());
		InterfaceRegistry.RegisterInterface(Audio::SourceOrientationInterface::GetInterface());
		InterfaceRegistry.RegisterInterface(Audio::ListenerOrientationInterface::GetInterface());
		InterfaceRegistry.RegisterInterface(Audio::SourceLocationInterface::GetInterface());
		InterfaceRegistry.RegisterInterface(Audio::ListenerLocationInterface::GetInterface());
		InterfaceRegistry.RegisterInterface(Audio::SourceOcclusionInterface::GetInterface());
		InterfaceRegistry.RegisterInterface(Audio::WorldTimeInterface::GetInterface());
	}
}

USoundClass* UAudioSettings::GetDefaultMediaSoundClass() const
{
	return DefaultMediaSoundClass;
}

USoundClass* UAudioSettings::GetDefaultSoundClass() const
{
	return DefaultSoundClass;
}

USoundConcurrency* UAudioSettings::GetDefaultSoundConcurrency() const
{
	return DefaultSoundConcurrency;
}

int32 UAudioSettings::GetHighestMaxChannels() const
{
	check(QualityLevels.Num() > 0);

	int32 HighestMaxChannels = -1;
	for (const FAudioQualitySettings& Settings : QualityLevels)
	{
		HighestMaxChannels = FMath::Max(Settings.MaxChannels, HighestMaxChannels);
	}

	return HighestMaxChannels;
}

FString UAudioSettings::FindQualityNameByIndex(int32 Index) const
{
	return QualityLevels.IsValidIndex(Index) ?
		   QualityLevels[Index].DisplayName.ToString() :
		   TEXT("");
}

ENGINE_API FDecodeCacheSettings UAudioSettings::GetDecodeCacheSettings() const
{
	if (UAudioPlatformSettings* Settings = UPlatformSettingsManager::Get().GetSettingsForPlatform<UAudioPlatformSettings>())
	{
		return Settings->DecodeCacheSettings;
	}
	return FDecodeCacheSettings();
}

#undef LOCTEXT_NAMESPACE

