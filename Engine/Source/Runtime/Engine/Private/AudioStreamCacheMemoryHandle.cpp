// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioStreamCacheMemoryHandle.h"
#include "ContentStreaming.h"

FAudioStreamCacheMemoryHandle::FAudioStreamCacheMemoryHandle(FName InFeatureName, uint64 InMemoryUseInBytes)
: FeatureName(InFeatureName)
, MemoryUseInBytes(InMemoryUseInBytes)
{
	if (MemoryUseInBytes != 0)
	{
		IStreamingManager::Get().GetAudioStreamingManager().AddMemoryCountedFeature(*this);
	}
}

FAudioStreamCacheMemoryHandle::~FAudioStreamCacheMemoryHandle()
{
	if (MemoryUseInBytes != 0)
	{
		IStreamingManager::Get().GetAudioStreamingManager().RemoveMemoryCountedFeature(*this);
	}
}

void FAudioStreamCacheMemoryHandle::ResetMemoryUseInBytes(uint64 InMemoryUseInBytes)
{
	if (MemoryUseInBytes != 0)
	{
		IStreamingManager::Get().GetAudioStreamingManager().RemoveMemoryCountedFeature(*this);
	}
		
	MemoryUseInBytes = InMemoryUseInBytes;

	if (MemoryUseInBytes != 0)
	{
		IStreamingManager::Get().GetAudioStreamingManager().AddMemoryCountedFeature(*this);
	}
}