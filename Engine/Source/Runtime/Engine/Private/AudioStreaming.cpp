// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AudioStreaming.cpp: Implementation of audio streaming classes.
=============================================================================*/

#include "AudioStreaming.h"
#include "Audio.h"
#include "Misc/CoreStats.h"
#include "DerivedDataCacheInterface.h"
#include "Serialization/MemoryReader.h"
#include "HAL/PlatformFile.h"
#include "Async/AsyncFileHandle.h"

static int32 SpoofFailedStreamChunkLoad = 0;
FAutoConsoleVariableRef CVarSpoofFailedStreamChunkLoad(
	TEXT("au.SpoofFailedStreamChunkLoad"),
	SpoofFailedStreamChunkLoad,
	TEXT("Forces failing to load streamed chunks.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 MaxConcurrentStreamsCvar = 0;
FAutoConsoleVariableRef CVarMaxConcurrentStreams(
	TEXT("au.MaxConcurrentStreams"),
	MaxConcurrentStreamsCvar,
	TEXT("Overrides the max concurrent streams.\n")
	TEXT("0: Not Overridden, >0 Overridden"),
	ECVF_Default);


/*------------------------------------------------------------------------------
	Streaming chunks from the derived data cache.
------------------------------------------------------------------------------*/

#if WITH_EDITORONLY_DATA

/** Initialization constructor. */
FAsyncStreamDerivedChunkWorker::FAsyncStreamDerivedChunkWorker(
	const FString& InDerivedDataKey,
	void* InDestChunkData,
	int32 InChunkSize,
	FThreadSafeCounter* InThreadSafeCounter,
	TFunction<void(bool)> InOnLoadCompleted
	)
	: DerivedDataKey(InDerivedDataKey)
	, DestChunkData(InDestChunkData)
	, ExpectedChunkSize(InChunkSize)
	, bRequestFailed(false)
	, ThreadSafeCounter(InThreadSafeCounter)
	, OnLoadCompleted(InOnLoadCompleted)
{
}

/** Retrieves the derived chunk from the derived data cache. */
void FAsyncStreamDerivedChunkWorker::DoWork()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FAsyncStreamDerivedChunkWorker::DoWork"), STAT_AsyncStreamDerivedChunkWorker_DoWork, STATGROUP_StreamingDetails);

	UE_LOGF(LogAudio, Verbose, "Start of ASync DDC Chunk read for key: %ls", *DerivedDataKey);

	TArray<uint8> DerivedChunkData;

	if (GetDerivedDataCacheRef().GetSynchronous(*DerivedDataKey, DerivedChunkData, TEXTVIEW("Unknown Audio")))
	{
		FMemoryReader Ar(DerivedChunkData, true);
		int32 ChunkSize = 0;
		int32 AudioDataSize = 0;
		Ar << ChunkSize;
		Ar << AudioDataSize;

		// Currently, the legacy streaming manager loads in the entire, zero padded chunk, while the cached streaming manager only reads the audio data itself.
		checkf(AudioDataSize == ExpectedChunkSize || ChunkSize == ExpectedChunkSize, TEXT("Neither the padded chunk size (%d) nor the actual audio data size (%d) was equivalent to the ExpectedSize(%d)"), ChunkSize, AudioDataSize, ExpectedChunkSize);
		Ar.Serialize(DestChunkData, ExpectedChunkSize);
	}
	else
	{
		bRequestFailed = true;
	}
	FPlatformMisc::MemoryBarrier();
	ThreadSafeCounter->Decrement();

	OnLoadCompleted(bRequestFailed);
	UE_LOGF(LogAudio, Verbose, "End of Async DDC Chunk Load. DDC Key: %ls", *DerivedDataKey);
}

#endif // #if WITH_EDITORONLY_DATA