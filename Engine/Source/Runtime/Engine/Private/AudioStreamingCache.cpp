// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AudioStreaming.cpp: Implementation of audio streaming classes.
=============================================================================*/

#include "AudioStreamingCache.h"
#include "AudioStreamCacheMemoryHandle.h"
#include "Async/Async.h"
#include "Audio.h"
#include "Audio/AudioDebug.h"
#include "Audio/AudioTimingLog.h"
#include "AudioCompressionSettingsUtils.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "Stats/StatsTrace.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Audio/AudioDebug.h"
#include "Audio/AudioDebugDraw.h"
#include "IO/IoBuffer.h"
#include "Templates/Greater.h"

DEFINE_LOG_CATEGORY(LogAudioStreamCaching);

TRACE_DECLARE_ATOMIC_INT_COUNTER(AudioStreamCache_Loads, TEXT("AudioStreamCache Loads"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(AudioStreamCache_Loads_Request, TEXT("AudioStreamCache Loads Request"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(AudioStreamCache_Loads_Playback, TEXT("AudioStreamCache Loads Playback"));

static int32 DebugMaxElementsDisplayCVar = 128;
FAutoConsoleVariableRef CVarDebugDisplayCaches(
	TEXT("au.streamcaching.MaxCachesToDisplay"),
	DebugMaxElementsDisplayCVar,
	TEXT("Sets the max amount of stream chunks to display on screen.\n")
	TEXT("n: Number of elements to display on screen."),
	ECVF_Default);

enum CacheDisplaySortMethod
{
	CDSM_RecentlyUsed,
	CDSM_LoadTime,
	CDSM_AccessCount,
	CDSM_RetainingHandleCount,
	CDSM_LoadStartTime,
};

static int32 DebugCacheDisplaySortMethod = 0;
FAutoConsoleVariableRef CVarDebugCacheDisplaySortMethod(
	TEXT("au.streamcaching.CacheDisplaySortMethod"),
	DebugCacheDisplaySortMethod,
	TEXT("What to sort on when showing the stream cache elements. Requries DEBUG_STREAM_CACHE.\n")
	TEXT("  0 (def): Recently Used.\n")
	TEXT("  1      : Load Time.\n")
	TEXT("  2      : Access Count.\n")
	TEXT("  3      : Retaining Handle Count.\n")
	TEXT("  4      : When Loaded.\n"),
	ECVF_Default);

static int32 KeepCacheMissBufferOnFlushCVar = 1;
FAutoConsoleVariableRef CVarKeepCacheMissBufferOnFlush(
	TEXT("au.streamcaching.KeepCacheMissBufferOnFlush"),
	KeepCacheMissBufferOnFlushCVar,
	TEXT("If set to 1, this will maintain the buffer of recorded cache misses after calling AudioMemReport. Otherwise, calling audiomemreport will flush all previous recorded cache misses.\n")
	TEXT("1: All cache misses from the whole session will show up in audiomemreport. 0: Only cache misses since the previous call to audiomemreport will show up in the current audiomemreport."),
	ECVF_Default);

static int32 ForceBlockForLoadCVar = 0;
FAutoConsoleVariableRef CVarForceBlockForLoad(
	TEXT("au.streamcaching.ForceBlockForLoad"),
	ForceBlockForLoadCVar,
	TEXT("When set to a nonzero value, blocks GetLoadedChunk until the disk read is complete.\n"),
	ECVF_Default);

static int32 TrimCacheWhenOverBudgetCVar = 1;
FAutoConsoleVariableRef CVarTrimCacheWhenOverBudget(
	TEXT("au.streamcaching.TrimCacheWhenOverBudget"),
	TrimCacheWhenOverBudgetCVar,
	TEXT("When set to a nonzero value, TrimMemory will be called in AddOrTouchChunk to ensure we never go over budget.\n"),
	ECVF_Default);
	
static int32 AlwaysLogCacheMissesCVar = 0;
FAutoConsoleVariableRef CVarAlwaysLogCacheMisses(
	TEXT("au.streamcaching.AlwaysLogCacheMisses"),
	AlwaysLogCacheMissesCVar,
	TEXT("When set to a nonzero value, all cache misses will be added to the audiomemreport.\n")
	TEXT("0: Don't log cache misses until au.streamcaching.StartProfiling is called. 1: Always log cache misses."),
	ECVF_Default);

static int32 ReadRequestPriorityCVar = 1;
FAutoConsoleVariableRef CVarReadRequestPriority(
	TEXT("au.streamcaching.ReadRequestPriority"),
	ReadRequestPriorityCVar,
	TEXT("This cvar sets the default request priority for audio chunks when Stream Caching is turned on.\n")
	TEXT("0: High, 1: Normal, 2: Below Normal, 3: Low, 4: Min"),
	ECVF_Default);

static int32 PlaybackRequestPriorityCVar = 0;
FAutoConsoleVariableRef CVarPlaybackRequestPriority(
	TEXT("au.streamcaching.PlaybackRequestPriority"),
	PlaybackRequestPriorityCVar,
	TEXT("This cvar sets the default request priority for audio chunks that are about to play back, but aren't in the cache.\n")
	TEXT("0: High, 1: Normal, 2: Below Normal, 3: Low, 4: Min"),
	ECVF_Default);

static int32 BlockForPendingLoadOnCacheOverflowCVar = 0;
FAutoConsoleVariableRef CVarBlockForPendingLoadOnCacheOverflow(
	TEXT("au.streamcaching.BlockForPendingLoadOnCacheOverflow"),
	BlockForPendingLoadOnCacheOverflowCVar,
	TEXT("This cvar sets the default request priority for audio chunks that are about to play back, but aren't in the cache.\n")
	TEXT("0: When we blow the cache we clear any soundwave retainers. 1:When we blow the cache we attempt to cancel a load in flight."),
	ECVF_Default);

static int32 NumSoundWavesToClearOnCacheOverflowCVar = 0;
FAutoConsoleVariableRef CVarNumSoundWavesToClearOnCacheOverflow(
	TEXT("au.streamcaching.NumSoundWavesToClearOnCacheOverflow"),
	NumSoundWavesToClearOnCacheOverflowCVar,
	TEXT("When set > 0, we will attempt to release retainers for only that many sounds every time we have a cache overflow.\n")
	TEXT("0: reset all retained sounds on cache overflow, >0: evict this many sounds on any cache overflow."),
	ECVF_Default);

static int32 EnableTrimmingRetainedAudioCVar = 1;
FAutoConsoleVariableRef CVarEnableTrimmingRetainedAudio(
	TEXT("au.streamcaching.EnableTrimmingRetainedAudio"),
	EnableTrimmingRetainedAudioCVar,
	TEXT("When set > 0, we will trim retained audio when the stream cache goes over the memory limit.\n")
	TEXT("0: never trims retained audio, >0: will trim retained audio."),
	ECVF_Default);
	
static float MemoryLimitTrimPercentageCVar = 0.1f;
FAutoConsoleVariableRef CVarMemoryLimitTrimPercentage(
	TEXT("au.streamcaching.MemoryLimitTrimPercentage"),
	MemoryLimitTrimPercentageCVar,
	TEXT("When set > 0.0, we will trim percentage of memory cache audio per trim call audio when the stream cache goes over the memory limit.\n")
	TEXT("0.0: trims only the amount needed to allocate a single chunk, >0: that percentage of memory limit."),
	ECVF_Default);

static float StreamCacheSizeOverrideMBCVar = 0.0f;
FAutoConsoleVariableRef CVarStreamCacheSizeOverrideMB(
	TEXT("au.streamcaching.StreamCacheSizeOverrideMB"),
	StreamCacheSizeOverrideMBCVar,
	TEXT("This cvar can be set to override the size of the cache.\n")
	TEXT("0: use cache size from project settings. n: the new cache size in megabytes."),
	ECVF_Default);

static int32 SaveAudioMemReportOnCacheOverflowCVar = 0;
FAutoConsoleVariableRef CVarSaveAudiomemReportOnCacheOverflow(
	TEXT("au.streamcaching.SaveAudiomemReportOnCacheOverflow"),
	SaveAudioMemReportOnCacheOverflowCVar,
	TEXT("When set to one, we print an audiomemreport when the cache has overflown.\n")
	TEXT("0: Disabled, 1: Enabled"),
	ECVF_Default);

static int32 DebugViewCVar = 2;
FAutoConsoleVariableRef CVarDebugView(
	TEXT("au.streamcaching.DebugView"),
	DebugViewCVar,
	TEXT("Controls Drawing the Debug Display for the Stream Cache.\n")
	TEXT("0: Legacy, 1: Default, 2: Averaged View, 3: High Detail View"),
	ECVF_Default);

static int32 SearchUsingChunkArrayCVar = 1;
FAutoConsoleVariableRef CVarSearchUsingChunkArray(
	TEXT("au.streamcaching.SearchUsingChunkArray"),
	SearchUsingChunkArrayCVar,
	TEXT("If performing an exhaustive search of the cache, use the chunk array instead of the LRU (we give up knowing how far down the cache an element was).\n")
	TEXT("0: Search using LRU (linked list). 1: Search using Chunk Pool (TArray)"),
	ECVF_Default);

static int32 EnableExhaustiveCacheSearchesCVar = 0;
FAutoConsoleVariableRef CVarEnableExhaustiveCacheSearches(
	TEXT("au.streamcaching.EnableExhaustiveCacheSearches"),
	EnableExhaustiveCacheSearchesCVar,
	TEXT("Enables an exhaustive search of the cache in FindElementForKey.\n")
	TEXT("0: Rely on chunk offset. 1: Search using linear search"),
	ECVF_Default);

static FAutoConsoleCommand GFlushAudioCacheCommand(
	TEXT("au.streamcaching.FlushAudioCache"),
	TEXT("This will flush any non retained audio from the cache when Stream Caching is enabled."),
	FConsoleCommandDelegate::CreateStatic(
		[]()
	{
		static constexpr uint64 NumBytesToFree = TNumericLimits<uint64>::Max() / 2;
		uint64 NumBytesFreed = IStreamingManager::Get().GetAudioStreamingManager().TrimMemory(NumBytesToFree);

		UE_LOGF(LogAudioStreamCaching, Display, "Audio Cache Flushed! %f megabytes free.", NumBytesFreed / (1024.0 * 1024.0));
	})
);


static const TCHAR* LexToString(const EAsyncIOPriorityAndFlags In)
{
#define CASE_TO_STRING(X) case X: return TEXT(#X)
	switch (In & AIOP_PRIORITY_MASK)
	{
		CASE_TO_STRING(AIOP_MIN);
		CASE_TO_STRING(AIOP_Low);
		CASE_TO_STRING(AIOP_BelowNormal);
		CASE_TO_STRING(AIOP_Normal);
		CASE_TO_STRING(AIOP_High);
		CASE_TO_STRING(AIOP_CriticalPath);
		default: return TEXT("Unknown");
	}
#undef CASE_TO_STRING
}

static EAsyncIOPriorityAndFlags ToIoPriority(const int32 InValue)
{
	switch (InValue)
	{
		case 4: return AIOP_MIN;
		case 3: return AIOP_Low;
		case 2: return AIOP_BelowNormal;
		case 1: return AIOP_Normal;
		case 0: return AIOP_High;
		default: break;
	}
	return AIOP_High;
}

static void AddLoadCounters(const EAsyncIOPriorityAndFlags InPriorityAndFlags, const int32 InValue)
{
	const EAsyncIOPriorityAndFlags Priority = InPriorityAndFlags & AIOP_PRIORITY_MASK;
	const EAsyncIOPriorityAndFlags ReadPriority = ToIoPriority(ReadRequestPriorityCVar);
	const EAsyncIOPriorityAndFlags PlaybackPriority = ToIoPriority(PlaybackRequestPriorityCVar);

	if (ReadPriority == Priority)
	{
		TRACE_COUNTER_ADD(AudioStreamCache_Loads_Request, InValue);
	}

	if (PlaybackPriority == Priority)
	{
		TRACE_COUNTER_ADD(AudioStreamCache_Loads_Playback, InValue);
	}
	
	TRACE_COUNTER_ADD(AudioStreamCache_Loads, InValue);
}

static FAutoConsoleCommand GResizeAudioCacheCommand(
	TEXT("au.streamcaching.ResizeAudioCacheTo"),
	TEXT("This will try to cull enough audio chunks to shrink the audio stream cache to the new size if neccessary, and keep the cache at that size."),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray< FString >& Args)
{
	if (Args.Num() < 1)
	{
		return;
	}

	const float InMB = FCString::Atof(*Args[0]);

	if (InMB <= 0.0f)
	{
		return;
	}

	static IConsoleVariable* StreamCacheSizeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.streamcaching.StreamCacheSizeOverrideMB"));
	check(StreamCacheSizeCVar);

	uint64 NewCacheSizeInBytes = ((uint64)(InMB * 1024)) * 1024;
	uint64 OldCacheSizeInBytes = ((uint64)(StreamCacheSizeCVar->GetFloat() * 1024)) * 1024;

	// TODO: here we delete the difference between the old cache size and the new cache size,
	// but we don't actually need to do this unless the cache is full.
	// In the future we can use our current cache usage to figure out how much we need to trim.
	if (NewCacheSizeInBytes < OldCacheSizeInBytes)
	{
		uint64 NumBytesToFree = OldCacheSizeInBytes - NewCacheSizeInBytes;
		IStreamingManager::Get().GetAudioStreamingManager().TrimMemory(NumBytesToFree);
	}

	StreamCacheSizeCVar->Set(InMB);

	UE_LOGF(LogAudioStreamCaching, Display, "Audio Cache Shrunk! Now set to be %f MB.", InMB);
})
);

static FAutoConsoleCommand GEnableProfilingAudioCacheCommand(
	TEXT("au.streamcaching.StartProfiling"),
	TEXT("This will start a performance-intensive profiling mode for this streaming manager. Profile stats can be output with audiomemreport."),
	FConsoleCommandDelegate::CreateStatic(
		[]()
{
	IStreamingManager::Get().GetAudioStreamingManager().SetProfilingMode(true);

	UE_LOGF(LogAudioStreamCaching, Display, "Enabled profiling mode on the audio stream cache.");
})
);

static FAutoConsoleCommand GDisableProfilingAudioCacheCommand(
	TEXT("au.streamcaching.StopProfiling"),
	TEXT("This will start a performance-intensive profiling mode for this streaming manager. Profile stats can be output with audiomemreport."),
	FConsoleCommandDelegate::CreateStatic(
		[]()
{
	IStreamingManager::Get().GetAudioStreamingManager().SetProfilingMode(false);

	UE_LOGF(LogAudioStreamCaching, Display, "Disabled profiling mode on the audio stream cache.");
})
);

FAudioChunkCache::FChunkKey::FChunkKey(const TSharedRef<const FSoundWaveData>& InSoundWave, uint32 InChunkIndex
#if WITH_EDITOR
	, uint32 InChunkRevision
#endif // #if WITH_EDITOR
)

	: ChunkIndex(InChunkIndex)
#if WITH_EDITOR
	, ChunkRevision(InChunkRevision)
#endif // #if WITH_EDITOR
{
	SoundWaveName = InSoundWave->GetFName();
	ObjectKey = InSoundWave->GetGUID();
}

FAudioChunkCache::FChunkKey::FChunkKey(const FName& InSoundWaveName , const FGuid& InSoundWaveObjectKey , uint32 InChunkIndex
#if WITH_EDITOR
		, uint32 InChunkRevision
#endif // #if WITH_EDITOR
)
	: SoundWaveName(InSoundWaveName)
	, ObjectKey(InSoundWaveObjectKey)
	, ChunkIndex(InChunkIndex)
#if WITH_EDITOR
	, ChunkRevision(InChunkRevision)
#endif // #if WITH_EDITOR
{
}


bool FAudioChunkCache::FChunkKey::operator==(const FChunkKey& Other) const
{
#if WITH_EDITOR
	return (SoundWaveName == Other.SoundWaveName) && (ObjectKey == Other.ObjectKey) && (ChunkIndex == Other.ChunkIndex) && (ChunkRevision == Other.ChunkRevision);
#else
	return (SoundWaveName == Other.SoundWaveName) && (ObjectKey == Other.ObjectKey) && (ChunkIndex == Other.ChunkIndex);
#endif
}

void FAudioChunkCache::FCacheElement::WaitForAsyncLoadCompletion(bool bCancel)
{
#if WITH_EDITORONLY_DATA
	if (DDCTask.IsValid() && !DDCTask->IsDone())
	{
		if (bCancel)
		{
			DDCTask->Cancel();
		}

		DDCTask->EnsureCompletion(false);
	}
#endif

	if (Read.IsPending())
	{
		if (bCancel)
		{
			Read.Cancel();
		}
		else
		{
			static constexpr uint32 WaitTimeMs = 1000;
			for (int32 Attempt = 0; Attempt < 10 && Read.WaitFor(WaitTimeMs); Attempt++)
			{
				UE_CLOGF(Attempt % 5 == 0, LogAudioStreamCaching, Warning,
					"Still Waiting for Sound: %ls:%i",*Key.SoundWaveName.ToString(), Key.ChunkIndex);
			}
		}
		Read = {};
	}
}

#if DEBUG_STREAM_CACHE
bool FAudioChunkCache::FCacheElement::IsBeingPlayed() const
{
	const int32 NumActiveConsumers = NumConsumers.GetValue();

	// if we 2 or more consumers, this chunk is being rendered.
	// if we have 1 consumer, and we aren't Retained, then this chunk is being rendered
	return (NumActiveConsumers > 1)
		|| (NumActiveConsumers && (DebugInfo.LoadingBehavior != ESoundWaveLoadingBehavior::RetainOnLoad));
}

void FAudioChunkCache::FCacheElement::UpdateDebugInfoLoadingBehavior()
{
	if(TSharedPtr<const FSoundWaveData> SoundWaveDataPtr = SoundWaveWeakPtr.Pin())
	{
		// Recursing in no longer needed at this point since the inherited loading behavior has already been cached by the time this information is needed
		DebugInfo.LoadingBehavior = SoundWaveDataPtr->GetLoadingBehavior();
		DebugInfo.bLoadingBehaviorExternallyOverriden = SoundWaveDataPtr->WasLoadingBehaviorOverridden();
	}
	else
	{
		DebugInfo.LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
		DebugInfo.bLoadingBehaviorExternallyOverriden = false;
		DebugInfo.bWaveDestroyed = true;
	}
}
#endif // #if DEBUG_STREAM_CACHE

uint32 FAudioChunkCache::FCacheElement::GetNumChunks() const
{
	if (TSharedPtr<const FSoundWaveData> SoundWaveDataPtr = SoundWaveWeakPtr.Pin())
	{
		return SoundWaveDataPtr->GetNumChunks();
	}
	
	return 0;
}

FStreamedAudioChunk* FAudioChunkCache::FCacheElement::GetChunk(uint32 InChunkIndex) const
{
	TSharedPtr<const FSoundWaveData> SoundWaveDataPtr = SoundWaveWeakPtr.Pin();

	// the Weakptr should be valid here since it's from a shared ptr up the stack
	if (ensure(SoundWaveDataPtr))
	{
		// This function shouldn't be called on audio marked "ForceInline."
		checkf(SoundWaveDataPtr->GetLoadingBehavior() != ESoundWaveLoadingBehavior::ForceInline, TEXT("Calling GetNumChunks on a FSoundWaveProxy that is Force-Inlined is not allowed! SoundWave: %s - %s")
			, *SoundWaveDataPtr->GetFName().ToString(), EnumToString(SoundWaveDataPtr->GetLoadingBehavior()));

		if (ensureMsgf(InChunkIndex < (uint32)SoundWaveDataPtr->GetNumChunks(), TEXT("Attempt retrieve chunk %d when only %d exist on sound wave\"%s\"."), InChunkIndex, SoundWaveDataPtr->GetNumChunks(), *SoundWaveDataPtr->GetFName().ToString()))
		{
			return &SoundWaveDataPtr->GetChunk(InChunkIndex);
		}
	}

	return nullptr;
}

#if WITH_EDITOR
bool FAudioChunkCache::FCacheElement::IsChunkStale()
{
	if(TSharedPtr<const FSoundWaveData> SoundWaveDataPtr = SoundWaveWeakPtr.Pin())
	{
		return !SoundWaveDataPtr.IsValid() || (SoundWaveDataPtr->GetCurrentChunkRevision() != Key.ChunkRevision);
	}

	return true;
}
#endif // #if WITH_EDITOR

void FAudioChunkCache::FCacheElement::ReleaseRetainedAudioOnSoundWave()
{
	if (TSharedPtr<const FSoundWaveData> SoundWaveDataPtr = SoundWaveWeakPtr.Pin())
	{
		UE_CLOGF(WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Release Retained Audio, chunk %d, ObjectKey=%ls", *WriteToString<128>(Key.SoundWaveName), Key.ChunkIndex, *WriteToString<40>(Key.ObjectKey));
		check(SoundWaveDataPtr.IsValid());
		SoundWaveDataPtr->ReleaseCompressedAudio();
	}
}

bool FAudioChunkCache::FCacheElement::IsSoundWaveRetainingAudio() const
{
	if (TSharedPtr<const FSoundWaveData> SoundWaveDataPtr = SoundWaveWeakPtr.Pin())
	{
		return SoundWaveDataPtr->IsRetainingAudio();
	}
	return false;
}


FCachedAudioStreamingManager::FCachedAudioStreamingManager(const FCachedAudioStreamingManagerParams& InitParams)
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);
	checkf(InitParams.Caches.Num() > 0, TEXT("FCachedAudioStreamingManager should be initialized with dimensions for at least one cache."));

	// const FAudioStreamCachingSettings& CacheSettings = FPlatformCompressionUtilities::GetStreamCachingSettingsForCurrentPlatform();
	for (const FCachedAudioStreamingManagerParams::FCacheDimensions& CacheDimensions : InitParams.Caches)
	{
		CacheArray.Emplace(CacheDimensions.MaxChunkSize, CacheDimensions.NumElements, CacheDimensions.MaxMemoryInBytes);
	}

	// Here we make sure our CacheArray is sorted from smallest MaxChunkSize to biggest, so that GetCacheForWave can scan through these caches to find the appropriate cache for the chunk size.
	CacheArray.Sort();
}

FCachedAudioStreamingManager::~FCachedAudioStreamingManager()
{
}

void FCachedAudioStreamingManager::UpdateResourceStreaming(float DeltaTime, bool bProcessEverything /*= false*/)
{
	// The cached audio streaming manager doesn't tick.
}

int32 FCachedAudioStreamingManager::BlockTillAllRequestsFinished(float TimeLimit, bool bLogResults)
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);

	// TODO: Honor TimeLimit and bLogResults. Since we cancel any in flight read requests, this should not spin out.
	for (FAudioChunkCache& Cache : CacheArray)
	{
		Cache.CancelAllPendingLoads();
	}

	return 0;
}

void FCachedAudioStreamingManager::CancelForcedResources()
{
	// Unused.
}

void FCachedAudioStreamingManager::NotifyLevelChange()
{
	// Unused.
}

void FCachedAudioStreamingManager::SetDisregardWorldResourcesForFrames(int32 NumFrames)
{
	// Unused.
}

void FCachedAudioStreamingManager::AddLevel(class ULevel* Level)
{
	// Unused.
}

void FCachedAudioStreamingManager::RemoveLevel(class ULevel* Level)
{
	// Unused.
}

void FCachedAudioStreamingManager::NotifyLevelOffset(class ULevel* Level, const FVector& Offset)
{
	// Unused.
}

void FCachedAudioStreamingManager::AddForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& SoundWave)
{
	// add the sound wave to the first cache
	if (ensure(CacheArray.Num() > 0))
	{
		CacheArray[0].AddForceInlineSoundWave(SoundWave);
	}
}

void FCachedAudioStreamingManager::RemoveForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& SoundWave)
{
	// remove the sound wave from the first cache
	if (ensure(CacheArray.Num() > 0))
	{
		CacheArray[0].RemoveForceInlineSoundWave(SoundWave);
	}
}

void FCachedAudioStreamingManager::AddMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature)
{
	// add memory count to the first cache
	if (ensure(CacheArray.Num() > 0))
	{
		CacheArray[0].AddMemoryCountedFeature(Feature);
	}
}

void FCachedAudioStreamingManager::RemoveMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature)
{
	// remove memory count from the first cache
	if (ensure(CacheArray.Num() > 0))
	{
		CacheArray[0].RemoveMemoryCountedFeature(Feature);
	}
}

void FCachedAudioStreamingManager::HandleStarvation(const TSharedRef<const FSoundWaveData>& SoundWaveData, const uint32 ChunkIndex, const uint32 InBytesProduced, const uint32 InBytesExpected) 
{
	// If this sound wave is managed by a cache, use that to get the chunk:
	if (FAudioChunkCache* Cache = GetCacheForWave(SoundWaveData))
	{
		Cache->HandleStarvation(SoundWaveData, ChunkIndex, InBytesProduced, InBytesExpected);
	}
}

FAudioChunkHandle FCachedAudioStreamingManager::GetLoadedChunk(const TSharedRef<const FSoundWaveData>& SoundWaveData, uint32 ChunkIndex, bool bBlockForLoad, bool bForImmediatePlayback) const
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);
	bBlockForLoad |= (ForceBlockForLoadCVar != 0);

	// If this sound wave is managed by a cache, use that to get the chunk:
	FAudioChunkCache* Cache = GetCacheForWave(SoundWaveData);
	if (Cache)
	{
		// With this code, the zeroth chunk should never get hit.
		checkf(ChunkIndex != 0, TEXT("Decoder tried to access the zeroth chunk through the streaming manager. Use USoundWave::GetZerothChunk() instead."));

		FAudioChunkCache::FChunkKey ChunkKey(
			  SoundWaveData
			, ChunkIndex
#if WITH_EDITOR
			, (uint32)SoundWaveData->GetCurrentChunkRevision()
#endif
		);

		if (!FAudioChunkCache::DoesKeyContainValidChunkIndex(ChunkKey, *SoundWaveData))
		{
			UE_LOGF(LogAudioStreamCaching, Warning, "Invalid Chunk Index %d Requested for Wave %ls!", ChunkIndex, *SoundWaveData->GetFName().ToString());
			return FAudioChunkHandle();
		}

		// The function call below increments the reference count to the internal chunk.
		uint64 LookupIDForChunk = Cache->GetCacheLookupIDForChunk(ChunkKey);
		TArrayView<uint8> LoadedChunk = Cache->GetChunk(ChunkKey, SoundWaveData, bBlockForLoad, (bForImmediatePlayback || bBlockForLoad), LookupIDForChunk);

		// Ensure that, if we requested a synchronous load of this chunk, we didn't fail to load said chunk.
		UE_CLOGF(bBlockForLoad && !LoadedChunk.GetData(), LogAudioStreamCaching, Display, "Synchronous load of chunk index %d for SoundWave %ls failed to return any data. Likely because the cache was blown.", ChunkIndex, *SoundWaveData->GetFName().ToString());

		// Set the updated cache offset for this chunk index.
		Cache->SetCacheLookupIDForChunk(ChunkKey, LookupIDForChunk);

		UE_CLOGF(!bBlockForLoad && !LoadedChunk.GetData(), LogAudioStreamCaching, Verbose, "GetLoadedChunk called for chunk index %d of SoundWave %ls when audio was not loaded yet. This will result in latency.", ChunkIndex, *SoundWaveData->GetFName().ToString());

		// Finally, if there's a chunk after this in the sound, request that it is in the cache.
		const int32 NextChunk = GetNextChunkIndex(SoundWaveData, ChunkIndex);

		if (NextChunk != INDEX_NONE)
		{
			FAudioChunkCache::FChunkKey NextChunkKey(
				  SoundWaveData
				, ((uint32)NextChunk)
#if WITH_EDITOR
				, (uint32)SoundWaveData->GetCurrentChunkRevision()
#endif
			);

			const uint64 LookupIDForNextChunk = Cache->AddOrTouchChunk(NextChunkKey, SoundWaveData, [](EAudioChunkLoadResult) {}, ENamedThreads::AnyThread, bForImmediatePlayback); 
			if (LookupIDForNextChunk == InvalidAudioStreamCacheLookupID)
			{
				UE_LOG(LogAudioStreamCaching, Warning, TEXT("Failed to add follow-up chunk for %s. This should not happen with our current TrimMemory() logic"), *SoundWaveData->GetFName().ToString());
			}
			else
			{
				Cache->SetCacheLookupIDForChunk(NextChunkKey, LookupIDForNextChunk);
			}
		}

		return BuildChunkHandle(LoadedChunk.GetData(), LoadedChunk.Num(), SoundWaveData, SoundWaveData->GetFName(), ChunkIndex, LookupIDForChunk);
	}
	else
	{
		ensureMsgf(false, TEXT("Failed to find cache for wave %s. Are you sure this is a streaming wave?"), *SoundWaveData->GetFName().ToString());
		return FAudioChunkHandle();
	}
}

FAudioChunkCache* FCachedAudioStreamingManager::GetCacheForWave(const TSharedRef<const FSoundWaveData>& InSoundWave) const
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);

	// We only cache chunks beyond the zeroth chunk of audio (which is inlined directly on the asset)
	if (InSoundWave->GetNumChunks() > 1)
	{
		const int32 SoundWaveChunkSize = InSoundWave->GetChunk(1).AudioDataSize;
		return GetCacheForChunkSize(SoundWaveChunkSize);
	}
	else
	{
		return nullptr;
	}
}

FAudioChunkCache* FCachedAudioStreamingManager::GetCacheForChunkSize(uint32 InChunkSize) const
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);
	// Iterate over our caches until we find the lowest MaxChunkSize cache this sound's chunks will fit into. 
	for (int32 CacheIndex = 0; CacheIndex < CacheArray.Num(); CacheIndex++)
	{
		check(CacheArray[CacheIndex].MaxChunkSize >= 0);
		if (InChunkSize <= ((uint32)CacheArray[CacheIndex].MaxChunkSize))
		{
			return const_cast<FAudioChunkCache*>(&CacheArray[CacheIndex]);
		}
	}

	// If we ever hit this, something may have wrong during cook.
	// Please check to make sure this platform's implementation of IAudioFormat honors the MaxChunkSize parameter passed into SplitDataForStreaming,
	// or that FStreamedAudioCacheDerivedDataWorker::BuildStreamedAudio() is passing the correct MaxChunkSize to IAudioFormat::SplitDataForStreaming.
	ensureMsgf(false, TEXT("Chunks in SoundWave are too large: %d bytes"), InChunkSize);
	return nullptr;
}

int32 FCachedAudioStreamingManager::GetNextChunkIndex(const TSharedRef<const FSoundWaveData>& InSoundWaveData, uint32 InCurrentChunkIndex)
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);

	// TODO: Figure out a way to tell whether this wave is looping or not. For now we always prime the first chunk
	// during the playback of the last chunk.
	// if(bNotLooping) return ((int32) CurrentChunkIndex) < (InSoundWave->GetNumChunks() - 1);
	const int32 NumChunksTotal = InSoundWaveData->GetNumChunks();
	if (NumChunksTotal <= 2)
	{
		// If there's only one chunk to cache (besides the zeroth chunk, which is inlined),
		// We don't need to load anything.
		return INDEX_NONE;
	}
	else if (InCurrentChunkIndex == (NumChunksTotal - 1))
	{
		// if we're on the last chunk, load the first chunk after the zeroth chunk.
		return 1;
	}
	else
	{
		// Otherwise, there's another chunk of audio after this one before the end of this file.
		return InCurrentChunkIndex + 1;
	}
}

void FCachedAudioStreamingManager::AddReferenceToChunk(const FAudioChunkHandle& InHandle)
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);
	FAudioChunkCache* Cache = GetCacheForChunkSize(InHandle.CachedDataNumBytes);
	check(Cache);

	const FAudioChunkCache::FChunkKey ChunkKey(
		  InHandle.CorrespondingWaveName
		, InHandle.CorrespondingWaveGuid
		, ((uint32)InHandle.ChunkIndex)
#if WITH_EDITOR
		, InHandle.ChunkRevision
#endif
	);

	Cache->AddNewReferenceToChunk(ChunkKey);
}

void FCachedAudioStreamingManager::RemoveReferenceToChunk(const FAudioChunkHandle& InHandle)
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);
	FAudioChunkCache* Cache = GetCacheForChunkSize(InHandle.CachedDataNumBytes);
	check(Cache);

	const FAudioChunkCache::FChunkKey ChunkKey(
		  InHandle.CorrespondingWaveName
		, InHandle.CorrespondingWaveGuid
		, ((uint32)InHandle.ChunkIndex)
#if WITH_EDITOR
		, InHandle.ChunkRevision
#endif
	);

	Cache->RemoveReferenceToChunk(ChunkKey);
}

bool FCachedAudioStreamingManager::RequestChunk(const TSharedRef<const FSoundWaveData>& SoundWaveData, uint32 ChunkIndex, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted, ENamedThreads::Type ThreadToCallOnLoadCompletedOn, bool bForImmediatePlayback)
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);
	FAudioChunkCache* Cache = GetCacheForWave(SoundWaveData);

	if (Cache)
	{
		FAudioChunkCache::FChunkKey ChunkKey(
			  SoundWaveData
			, ChunkIndex
#if WITH_EDITOR
			, (uint32)SoundWaveData->GetCurrentChunkRevision()
#endif
		);

		uint64 LookupIDForChunk = Cache->AddOrTouchChunk(ChunkKey, SoundWaveData, OnLoadCompleted, ThreadToCallOnLoadCompletedOn, bForImmediatePlayback);

		// Update the map entry through the streaming cache
		Cache->SetCacheLookupIDForChunk(ChunkKey, LookupIDForChunk);
		return LookupIDForChunk != InvalidAudioStreamCacheLookupID;
	}
	else
	{
		// This can hit if an out of bounds chunk was requested, or the zeroth chunk was requested from the streaming manager.
		// Also legitimately reached during USoundCue::PostLoad when FSoundWaveProxy::PublishedSoundWaveData has not yet been
		// Publish'd in sync with USoundWave::WorkingSoundWaveData — the gate at the call site reads working data, while the
		// data ref we get from the proxy is still the default-initialized published copy. Silently return false in that case.
		return false;
	}
}

FAudioChunkCache::FAudioChunkCache(uint32 InMaxChunkSize, uint32 NumChunks, uint64 InMemoryLimitInBytes)
	: MaxChunkSize(InMaxChunkSize)
	, MostRecentElement(nullptr)
	, LeastRecentElement(nullptr)
	, ChunksInUse(0)
	, MemoryCounterBytes(0)
	, MemoryLimitBytes(InMemoryLimitInBytes)
	, ForceInlineMemoryCounterBytes(0)
	, FeatureMemoryCounterBytes(0)
	, bLogCacheMisses(false)
{
	check(NumChunks > 0);
	CachePool.Reset(NumChunks);
	for (uint32 Index = 0; Index < NumChunks; Index++)
	{
		CachePool.Emplace(Index);
	}
	CacheOverflowCount.Set(0);

	if (MemoryLimitBytes == 0)
	{
		UE_LOGF(LogAudioStreamCaching, Display, "Audio stream cache size is 0 bytes. Audio will not play. To resolve this issue, set \'CacheSizeKB\' to a non-zero value.");
	}
}

FAudioChunkCache::~FAudioChunkCache()
{
	LLM_SCOPE(ELLMTag::AudioStreamCache);

	CancelAllPendingLoads();
	
	// Ensure all in-flight loads are cancelled and waited on before the pool is destroyed.
	CachePool.Reset();
	check(NumberOfLoadsInFlight.GetValue() == 0);
}

uint64 FAudioChunkCache::AddOrTouchChunk(const FChunkKey& InKey, const TSharedPtr<FSoundWaveData>& InSoundWaveData, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted, ENamedThreads::Type CallbackThread, bool bNeededForPlayback)
{
	if (!InSoundWaveData.IsValid())
	{
		ExecuteOnLoadCompleteCallback(EAudioChunkLoadResult::ChunkOutOfBounds, OnLoadCompleted, CallbackThread);
		return InvalidAudioStreamCacheLookupID;
	}

	TSharedRef<const FSoundWaveData> SoundWaveDataRef = InSoundWaveData.ToSharedRef();

	return AddOrTouchChunk(InKey, SoundWaveDataRef, OnLoadCompleted, CallbackThread, bNeededForPlayback);
}

uint64 FAudioChunkCache::AddOrTouchChunk(const FChunkKey& InKey, const TSharedRef<const FSoundWaveData>& InSoundWaveData, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted, ENamedThreads::Type CallbackThread, bool bNeededForPlayback)
{
	// Update cache limit if needed.
	if (!FMath::IsNearlyZero(StreamCacheSizeOverrideMBCVar) && StreamCacheSizeOverrideMBCVar > 0.0f)
	{
		MemoryLimitBytes = ((uint64)(StreamCacheSizeOverrideMBCVar * 1024)) * 1024;
	}

	if (!DoesKeyContainValidChunkIndex(InKey, *InSoundWaveData))
	{
		ensure(false);
		ExecuteOnLoadCompleteCallback(EAudioChunkLoadResult::ChunkOutOfBounds, OnLoadCompleted, CallbackThread);
		return InvalidAudioStreamCacheLookupID;
	}

	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

	FCacheElement* FoundElement = FindElementForKey(InKey);
	if (FoundElement)
	{
		TouchElement(FoundElement);
		if (FoundElement->Read.IsPending())
		{
			// Bump priority if we've changed from loading to playing.
			const EAsyncIOPriorityAndFlags Priority = GetAsyncPriorityForChunk(InKey, bNeededForPlayback);
			if (FoundElement->Priority < Priority)
			{
				UE_CLOGF(FoundElement->WantsFilteredLogging(), LogAudioStreamCaching, Verbose,
					"[%ls] - Bumping priority of existing read, now we're in playback), chunk %d, ObjectKey=%ls",
						*WriteToString<128>(FoundElement->Key.SoundWaveName), FoundElement->Key.ChunkIndex, *WriteToString<40>(FoundElement->Key.ObjectKey));

				FoundElement->Read.UpdatePriority(Priority);
				FoundElement->Priority = Priority;
			}
		}
			
		if (FoundElement->bIsLoaded)
		{
			ExecuteOnLoadCompleteCallback(EAudioChunkLoadResult::AlreadyLoaded, OnLoadCompleted, CallbackThread);
		}

	UE_CLOGF(FoundElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - AddOrTouchChunk(found), chunk %d, ObjectKey=%ls", *WriteToString<128>(FoundElement->Key.SoundWaveName), FoundElement->Key.ChunkIndex, *WriteToString<40>(FoundElement->Key.ObjectKey));

#if DEBUG_STREAM_CACHE
		FoundElement->DebugInfo.NumTimesTouched++;

		FoundElement->UpdateDebugInfoLoadingBehavior();
#endif

		return FoundElement->CacheLookupID;
	}
	else
	{
		FCacheElement* CacheElement = InsertChunk(InKey, InSoundWaveData);

		if (!CacheElement)
		{
			ExecuteOnLoadCompleteCallback(EAudioChunkLoadResult::CacheBlown, OnLoadCompleted, CallbackThread);
			return InvalidAudioStreamCacheLookupID;
		}

#if DEBUG_STREAM_CACHE
		CacheElement->DebugInfo.bWasCacheMiss = bNeededForPlayback;
		CacheElement->UpdateDebugInfoLoadingBehavior();
#endif
		if (const FStreamedAudioChunk* Chunk = CacheElement->GetChunk(InKey.ChunkIndex))
		{
			int32 ChunkDataSize = Chunk->AudioDataSize;

			const uint64 MemoryUsageBytes = GetCurrentMemoryUsageBytes() + ChunkDataSize;
			if (TrimCacheWhenOverBudgetCVar != 0 && MemoryUsageBytes > MemoryLimitBytes)
			{
				uint64 MemoryToTrim = 0;
				if (MemoryLimitTrimPercentageCVar > 0.0f)
				{
					MemoryToTrim = MemoryLimitBytes * FMath::Min(MemoryLimitTrimPercentageCVar, 1.0f);
				}
				else
				{
					MemoryToTrim = MemoryUsageBytes - MemoryLimitBytes;
				}

				TrimMemory(MemoryToTrim, true);
			}
		}

		KickOffAsyncLoad(CacheElement, InKey, OnLoadCompleted, CallbackThread, bNeededForPlayback);

		if (bNeededForPlayback && (bLogCacheMisses || AlwaysLogCacheMissesCVar))
		{
			// We missed 
			const uint32 TotalNumChunksInWave = CacheElement->GetNumChunks();

			FCacheMissInfo CacheMissInfo = { InKey.SoundWaveName, InKey.ChunkIndex, TotalNumChunksInWave, false };
			CacheMissQueue->Enqueue(MoveTemp(CacheMissInfo));
		}

		return CacheElement->CacheLookupID;
	}
}

TArrayView<uint8> FAudioChunkCache::GetChunk(const FChunkKey& InKey, const TSharedPtr<FSoundWaveData>& InSoundWavePtr, bool bBlockForLoadCompletion, bool bNeededForPlayback, uint64& OutCacheOffset)
{
	if (ensure(InSoundWavePtr.IsValid()))
	{
		TSharedRef<const FSoundWaveData> SoundWaveDataRef = InSoundWavePtr.ToSharedRef();
			return GetChunk(InKey, SoundWaveDataRef, bBlockForLoadCompletion, bNeededForPlayback, OutCacheOffset);
	}

	return {};
}

TArrayView<uint8> FAudioChunkCache::GetChunk(const FChunkKey& InKey, const TSharedRef<const FSoundWaveData>& InSoundWavePtr, bool bBlockForLoadCompletion, bool bNeededForPlayback, uint64& OutCacheOffset)
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());
	FCacheElement* FoundElement = FindElementForKey(InKey);
	if (FoundElement)
	{
		UE_CLOGF(FoundElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - GetChunk(found), chunk %d, ObjectKey=%ls", *WriteToString<128>(FoundElement->Key.SoundWaveName), FoundElement->Key.ChunkIndex, *WriteToString<40>(FoundElement->Key.ObjectKey));
		OutCacheOffset = FoundElement->CacheLookupID;
		TouchElement(FoundElement);
		if (FoundElement->IsLoadInProgress())
		{
			if (bBlockForLoadCompletion)
			{
				FoundElement->WaitForAsyncLoadCompletion(false);
			}
			else
			{
				return TArrayView<uint8>();
			}
		}


		// If this value is ever negative, it means that we're decrementing more than we're incrementing:
		check(FoundElement->NumConsumers.GetValue() >= 0);
		FoundElement->NumConsumers.Increment();
		return TArrayView<uint8>(FoundElement->ChunkData, FoundElement->ChunkDataSize);
	}
	else
	{
		// If we missed it, kick off a new load with it.
		FoundElement = InsertChunk(InKey, InSoundWavePtr);
		if (!FoundElement)
		{
			OutCacheOffset = InvalidAudioStreamCacheLookupID;
			UE_LOGF(LogAudioStreamCaching, Display, "GetChunk failed to find an available chunk slot in the cache, likely because the cache is blown.");
			return TArrayView<uint8>();
		}

		OutCacheOffset = FoundElement->CacheLookupID;

// In cooked / packaged builds we need to retrieve the data from the pak file
// if we are running with WITH_EDITORONLY_DATA, then this data has already been 
// accessed via the DDC, and the underlying FByteBulk we access below has been 
// cleared out
#if WITH_EDITORONLY_DATA
		bBlockForLoadCompletion = false;
#endif // #if !WITH_EDITORONLY_DATA

		if (bBlockForLoadCompletion)
		{
			if (FStreamedAudioChunk* Chunk = FoundElement->GetChunk(InKey.ChunkIndex))
			{
				UE_CLOGF(FoundElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - GetChunk(blocking, hit), chunk %d, ObjectKey=%ls", *WriteToString<128>(FoundElement->Key.SoundWaveName), FoundElement->Key.ChunkIndex, *WriteToString<40>(FoundElement->Key.ObjectKey));

				int32 ChunkAudioDataSize = Chunk->AudioDataSize;
#if DEBUG_STREAM_CACHE
				const float SampleRate = InSoundWavePtr->GetSampleRate();
				const int32 NumChunks = FoundElement->GetNumChunks();
				const uint32 OffsetInFrames = Chunk->SeekOffsetInAudioFrames;
				FoundElement->DebugInfo.OffsetInSeconds = OffsetInFrames != INDEX_NONE ? OffsetInFrames / SampleRate : -1.f;
				FoundElement->DebugInfo.NumTotalChunks = NumChunks - 1; //(exclude chunk 0)
				FoundElement->DebugInfo.TimeLoadStartedCycles = FPlatformTime::Cycles64();
				FoundElement->DebugInfo.TimeLoadEndedCycles = FoundElement->DebugInfo.TimeLoadStartedCycles;
				FoundElement->DebugInfo.TimeToLoadMs = 0;
#endif
				*MemoryCounterBytes -= FoundElement->ChunkDataSize;

				{
					LLM_SCOPE(ELLMTag::AudioStreamCacheCompressedData);

					// Reallocate our chunk data This allows us to shrink if possible.
					FoundElement->ChunkData = (uint8*)FMemory::Realloc(FoundElement->ChunkData, ChunkAudioDataSize);
					void* DataDestPtr = FoundElement->ChunkData;
					const bool Result = Chunk->GetCopy(&DataDestPtr);

					if (!Result)
					{
						UE_LOGF(LogAudioStreamCaching, Warning, "Failed to retrieve chunk data from Bulk Data for soundwave: %ls", *InKey.SoundWaveName.ToString());
						return TArrayView<uint8>();
					}
				}

				*MemoryCounterBytes += ChunkAudioDataSize;

				// Populate key and DataSize. The async read request was set up to write directly into CacheElement->ChunkData.
				FoundElement->Key = InKey;
				FoundElement->ChunkDataSize = ChunkAudioDataSize;
				FoundElement->bIsLoaded = true;

				// If this value is ever negative, it means that we're decrementing more than we're incrementing:
				if (ensureMsgf(FoundElement->NumConsumers.GetValue() >= 0, TEXT("NumConsumers was negative for FoundElement. Reseting to 1")))
				{
					FoundElement->NumConsumers.Increment();
				}
				else
				{
					FoundElement->NumConsumers.Set(1);
				}

				return TArrayView<uint8>(FoundElement->ChunkData, ChunkAudioDataSize);
			}
			else
			{
				UE_LOGF(LogAudioStreamCaching, Error, "Failed to get chunk %d from soundwave: %ls", InKey.ChunkIndex, *InKey.SoundWaveName.ToString());
			}
		}
		else
		{
			KickOffAsyncLoad(FoundElement, InKey, [](EAudioChunkLoadResult InResult) {}, ENamedThreads::AnyThread, bNeededForPlayback);
		}

		if (bLogCacheMisses && !bBlockForLoadCompletion)
		{
			// Chunks missing. Log this as a miss.
			const uint32 TotalNumChunksInWave = FoundElement->GetNumChunks();
			FCacheMissInfo CacheMissInfo = { InKey.SoundWaveName, InKey.ChunkIndex, TotalNumChunksInWave, false };
			CacheMissQueue->Enqueue(MoveTemp(CacheMissInfo));
		}

		// We missed, return an empty array view.
		return TArrayView<uint8>();
	}
}

void FAudioChunkCache::AddNewReferenceToChunk(const FChunkKey& InKey)
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());
	FCacheElement* FoundElement = FindElementForKey(InKey);
	if (ensure(FoundElement))
	{
		// If this value is ever negative, it means that we're decrementing more than we're incrementing:
		check(FoundElement->NumConsumers.GetValue() >= 0);
		FoundElement->NumConsumers.Increment();
	}
}

void FAudioChunkCache::RemoveReferenceToChunk(const FChunkKey& InKey)
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());
	FCacheElement* FoundElement = FindElementForKey(InKey);
	if (ensure(FoundElement))
	{
		// If this value is ever less than 1 when we hit this code, it means that we're decrementing more than we're incrementing:
		check(FoundElement->NumConsumers.GetValue() >= 1);
		FoundElement->NumConsumers.Decrement();
	}
}

void FAudioChunkCache::ClearCache()
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());
	const uint32 NumChunks = CachePool.Num();

	UE_LOGF(LogAudioStreamCaching, Verbose, "Clearing Cache");

	CachePool.Reset(NumChunks);
	check(NumberOfLoadsInFlight.GetValue() == 0);

	for (uint32 Index = 0; Index < NumChunks; Index++)
	{
		CachePool.Emplace(Index);
	}

	MostRecentElement = nullptr;
	LeastRecentElement = nullptr;
	ChunksInUse = 0;
}

void FAudioChunkCache::AddForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& InSoundWaveData)
{
	ensureMsgf(InSoundWaveData->GetLoadingBehavior() == ESoundWaveLoadingBehavior::ForceInline, 
		TEXT("AudioStreamingCache::AddForceInlineSoundWave: Attempted to add SoundWave not set to ForceInline: %s"), *InSoundWaveData->GetFName().ToString());

	if (InSoundWaveData->GetLoadingBehavior() != ESoundWaveLoadingBehavior::ForceInline)
	{
		return;
	}
	
	FName Format = InSoundWaveData->GetRuntimeFormat();
	FByteBulkData* Data = InSoundWaveData->GetCompressedData(Format);
	int64 MemoryCount = Data ? Data->GetBulkDataSize() : 0;
	int32 RefCount = 0;
	{
		FScopeLock Lock(&SoundWaveMemoryTrackerCritSec.Get());
		FSoundWaveMemoryTracker& Tracker = SoundWaveTracker.FindOrAdd(&InSoundWaveData.Get());
		checkf(Tracker.RefCount >= 0, TEXT("AudioStreamCache::AddForceInlineSoundWave: ref count for Added sound wave is negative!: %s"), *InSoundWaveData->GetFName().ToString());
		RefCount = ++Tracker.RefCount;
		if (RefCount == 1)
		{
			// set the tracker memory count to that of the sound wave
			Tracker.MemoryCount = MemoryCount;
		}
		else
		{
			// use the memory count set by the tracker.
			MemoryCount = Tracker.MemoryCount;
		}
	}

	// we only increment memory count for the first time the sound wave is added
	if (RefCount == 1)
	{
		*ForceInlineMemoryCounterBytes += MemoryCount;
		const uint64 MemoryUsageBytes = GetCurrentMemoryUsageBytes();
		if (TrimCacheWhenOverBudgetCVar != 0 && MemoryUsageBytes > MemoryLimitBytes)
		{
			uint64 MemoryToTrim = 0;
			if (MemoryLimitTrimPercentageCVar > 0.0f)
			{
				MemoryToTrim = MemoryLimitBytes * FMath::Min(MemoryLimitTrimPercentageCVar, 1.0f);
			}
			else
			{
				MemoryToTrim = MemoryUsageBytes - MemoryLimitBytes;
			}

			TrimMemory(MemoryToTrim, true);
		}
	}
}

void FAudioChunkCache::RemoveForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& InSoundWaveData)
{
	ensureMsgf(InSoundWaveData->GetLoadingBehavior() == ESoundWaveLoadingBehavior::ForceInline,
		TEXT("AudioStreamingCache::RemoveForceInlineSoundWave: Attempted to remove SoundWave not set to ForceInline: %s"), *InSoundWaveData->GetFName().ToString());

	if (InSoundWaveData->GetLoadingBehavior() != ESoundWaveLoadingBehavior::ForceInline)
	{ 
		return;
	}

	int64 MemoryCount = 0;
	int32 RefCount = 0;

	// scope lock
	{
		FScopeLock Lock(&SoundWaveMemoryTrackerCritSec.Get());

		FSoundWaveMemoryTracker* Tracker = SoundWaveTracker.Find(&InSoundWaveData.Get());
		checkf(Tracker != nullptr, TEXT("AudioStreamCache::RemoveForceInlineSoundWave: Attempted to remove SoundWave that was never added, or has already been removed: %s"), *InSoundWaveData->GetFName().ToString());
		checkf(Tracker->RefCount > 0, TEXT("AudioStreamCache::RemoveForceInlineSoundWve: Attempted to remove SoundWave that has a ref count of zero or less. Something has gone horribly wrong: %s"), *InSoundWaveData->GetFName().ToString());
		MemoryCount = Tracker->MemoryCount;
		RefCount = --Tracker->RefCount;
		// use the memory count we cached from the last sound wave add
		if (RefCount == 0)
		{
			SoundWaveTracker.Remove(&InSoundWaveData.Get());
		}
	}
	
	if (RefCount == 0)
	{
		*ForceInlineMemoryCounterBytes -= MemoryCount;
	}
}

void FAudioChunkCache::AddMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature)
{
	UE_LOGF(LogAudioStreamCaching, Log, "Adding Memory Counted Feature (%ls) Memory Usage: %d bytes", *Feature.GetFeatureName().ToString(), (int32)Feature.GetMemoryUseInBytes());
	const uint64 OldMemoryCount = FeatureMemoryCounterBytes->AddExchange(Feature.GetMemoryUseInBytes());
	UE_LOGF(LogAudioStreamCaching, Log, "Total Memory Usage for all features: %d -> %d bytes", (int32)OldMemoryCount, (int32)FeatureMemoryCounterBytes->Load());
	const uint64 MemoryUsageBytes = GetCurrentMemoryUsageBytes();
	if (TrimCacheWhenOverBudgetCVar != 0 && MemoryUsageBytes > MemoryLimitBytes)
	{
		uint64 MemoryToTrim = 0;
		if (MemoryLimitTrimPercentageCVar > 0.0f)
		{
			MemoryToTrim = MemoryLimitBytes * FMath::Min(MemoryLimitTrimPercentageCVar, 1.0f);
		}
		else
		{
			MemoryToTrim = MemoryUsageBytes - MemoryLimitBytes;
		}

		TrimMemory(MemoryToTrim, true);
	}
}

void FAudioChunkCache::RemoveMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature)
{
	UE_LOGF(LogAudioStreamCaching, Log, "Removing Memory Counted Feature (%ls) Memory Usage: %d", *Feature.GetFeatureName().ToString(), (int32)Feature.GetMemoryUseInBytes());
	checkf(FeatureMemoryCounterBytes->Load() >= Feature.GetMemoryUseInBytes(), TEXT("Count (%llu) < Remove (%llu)"), FeatureMemoryCounterBytes->Load(), Feature.GetMemoryUseInBytes());
	const uint32 OldMemoryCount = FeatureMemoryCounterBytes->SubExchange(Feature.GetMemoryUseInBytes());
	UE_LOGF(LogAudioStreamCaching, Log, "Total Memory Usage for all features: %d -> %d", (int32)OldMemoryCount, (int32)FeatureMemoryCounterBytes->Load());
}

uint64 FAudioChunkCache::TrimMemory(uint64 BytesToFree, bool bInAllowRetainedChunkTrimming)
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

	if (!MostRecentElement || MostRecentElement->LessRecentElement == nullptr)
	{
		return 0;
	}

	FCacheElement* CurrentElement = LeastRecentElement;

	// In order to avoid cycles, we always leave at least two chunks in the cache.
	const FCacheElement* ElementToStopAt = MostRecentElement->LessRecentElement;

	int32 NumElementsEvicted = 0;
	uint64 BytesFreed = 0;
	while (CurrentElement != ElementToStopAt && BytesFreed < BytesToFree)
	{
		if (CurrentElement->CanEvictChunk() && CurrentElement->ChunkDataSize != 0)
		{
			UE_CLOGF(CurrentElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Evicting for trim, chunk %d, ObjectKey=%ls", *WriteToString<128>(CurrentElement->Key.SoundWaveName), CurrentElement->Key.ChunkIndex, *WriteToString<40>(CurrentElement->Key.ObjectKey));

			uint32 ChunkSize = CurrentElement->ChunkDataSize; 
			BytesFreed += ChunkSize;
			*MemoryCounterBytes -= ChunkSize;

			// Empty the chunk data and invalidate the key.
			check(CurrentElement->ChunkData);
			LLM_SCOPE(ELLMTag::AudioStreamCacheCompressedData);
			FMemory::Free(CurrentElement->ChunkData);

			CurrentElement->ChunkData = nullptr;
			CurrentElement->ChunkDataSize = 0;
			CacheLookupIdMap.Remove(CurrentElement->Key);
			CurrentElement->Key = FChunkKey();

#if DEBUG_STREAM_CACHE
			// Reset debug info:
			CurrentElement->DebugInfo.Reset();
#endif
			NumElementsEvicted++;
		}

		// Important to note that we don't actually relink chunks here,
		// So by trimming memory we are not moving chunks up the recency list.
		CurrentElement = CurrentElement->MoreRecentElement;
	}

	uint64 RetainedBytesFreed = 0;
	uint32 NumRetainedElementsEvicted = 0;
	// If we have run out of non-retained and in-flight load audio chunks to trim, eat into the 
	if (bInAllowRetainedChunkTrimming && EnableTrimmingRetainedAudioCVar > 0 && BytesFreed < BytesToFree)
	{
		UE_LOG(LogAudioStreamCaching, Verbose, TEXT("TrimMemory: Num Non-Retained Elements Evicted: %d. Non-Retained Bytes Freed: %" UINT64_FMT), NumElementsEvicted, BytesFreed);

		CurrentElement = LeastRecentElement;
		ElementToStopAt = MostRecentElement->LessRecentElement;
		while (CurrentElement != ElementToStopAt && BytesFreed < BytesToFree)
		{
			if (CurrentElement->ChunkDataSize != 0 && CurrentElement->IsSoundWaveRetainingAudio())
			{
				// Directly release the retained audio (TODO: this is on the audio thread right?)
				CurrentElement->ReleaseRetainedAudioOnSoundWave();
				if (CurrentElement->CanEvictChunk())
				{
					UE_CLOGF(CurrentElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Evicting for trim, chunk %d, ObjectKey=%ls", *WriteToString<128>(CurrentElement->Key.SoundWaveName), CurrentElement->Key.ChunkIndex, *WriteToString<40>(CurrentElement->Key.ObjectKey));
					uint32 ChunkSize = CurrentElement->ChunkDataSize;
					BytesFreed += ChunkSize;
					RetainedBytesFreed += ChunkSize;
					*MemoryCounterBytes -= ChunkSize;

					// Empty the chunk data and invalidate the key.
					check(CurrentElement->ChunkData);

					LLM_SCOPE(ELLMTag::AudioStreamCacheCompressedData);
					FMemory::Free(CurrentElement->ChunkData);
					CurrentElement->ChunkData = nullptr;
					CurrentElement->ChunkDataSize = 0;
					CacheLookupIdMap.Remove(CurrentElement->Key);
					CurrentElement->Key = FChunkKey();

#if DEBUG_STREAM_CACHE
					// Reset debug info:
					CurrentElement->DebugInfo.Reset();
#endif
					NumElementsEvicted++;
					NumRetainedElementsEvicted++;
				}
			}

			CurrentElement = CurrentElement->MoreRecentElement;
		}

		UE_LOG(LogAudioStreamCaching, Verbose, TEXT("TrimMemory: Num Retained Elements Evicted: %d. Retained Bytes Freed: %" UINT64_FMT), NumRetainedElementsEvicted, RetainedBytesFreed);
	}

	UE_LOG(LogAudioStreamCaching, Verbose, TEXT("TrimMemory: Total Num Elements Evicted: %d. Total Bytes Freed: %" UINT64_FMT), NumElementsEvicted, BytesFreed);

	return BytesFreed;
}

void FAudioChunkCache::BlockForAllPendingLoads() const
{
	bool bLoadInProgress = false;

	float TimeStarted = FPlatformTime::Seconds();

	do
	{
		// If we did find an in flight async load,
		// sleep to let other threads complete this task.
		if (bLoadInProgress)
		{
			float TimeSinceStarted = FPlatformTime::Seconds() - TimeStarted;
			UE_LOGF(LogAudioStreamCaching, Log, "Waited %f seconds for async audio chunk loads.", TimeSinceStarted);
			FPlatformProcess::Sleep(0.0f);
		}

		{
			FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

			// Iterate through every element until we find one with a load in progress.
			const FCacheElement* CurrentElement = MostRecentElement;
			while (CurrentElement != nullptr)
			{
				bLoadInProgress |= CurrentElement->IsLoadInProgress();
				CurrentElement = CurrentElement->LessRecentElement;
			}
		}
	} while (bLoadInProgress);
}

void FAudioChunkCache::CancelAllPendingLoads()
{
	TArray<FCacheElement*> ToCancel;
	{
		// Lock and collect all elements.
		FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());
		FCacheElement* Current = MostRecentElement;
		while (Current != nullptr)
		{
			ToCancel.Add(Current);
			Current = Current->LessRecentElement;
		}
	}

	// Wait for completion outside the lock.
	for (FCacheElement* Elem : ToCancel)
	{
		Elem->WaitForAsyncLoadCompletion(/*bCancel=*/true);
	}
}

uint64 FAudioChunkCache::ReportCacheSize()
{
	const uint32 NumChunks = CachePool.Num();

	return MaxChunkSize * NumChunks;
}

void FAudioChunkCache::BeginLoggingCacheMisses()
{
	bLogCacheMisses = true;
}

void FAudioChunkCache::StopLoggingCacheMisses()
{
	bLogCacheMisses = false;
}

FString FAudioChunkCache::FlushCacheMissLog()
{
	FString ConcatenatedCacheMisses;
	ConcatenatedCacheMisses.Append(TEXT("All Cache Misses:\nSoundWave:\t, ChunkIndex\n"));

	struct FMissedChunk
	{
		FName SoundWaveName;
		uint32 ChunkIndex;
		int32 MissCount;
	};

	struct FCacheMissSortPredicate
	{
		FORCEINLINE bool operator()(const FMissedChunk& A, const FMissedChunk& B) const
		{
			// Sort from highest miss count to lowest.
			return A.MissCount > B.MissCount;
		}
	};

	TMap<FCacheMissEntry, int32> CacheMissCount;

	TQueue<FCacheMissInfo> BackupQueue;

	FCacheMissInfo CacheMissInfo;
	while (CacheMissQueue->Dequeue(CacheMissInfo))
	{
		ConcatenatedCacheMisses.Append(CacheMissInfo.SoundWaveName.ToString());
		ConcatenatedCacheMisses.Append(TEXT("\t, "));
		ConcatenatedCacheMisses.AppendInt(CacheMissInfo.ChunkIndex);
		ConcatenatedCacheMisses.Append(TEXT("\n"));

		FCacheMissEntry CacheMissEntry(CacheMissInfo.SoundWaveName, CacheMissInfo.ChunkIndex);

		int32& MissCount = CacheMissCount.FindOrAdd(CacheMissEntry);
		MissCount++;

		if (KeepCacheMissBufferOnFlushCVar)
		{
			BackupQueue.Enqueue(CacheMissInfo);
		}
	}

	// Sort our cache miss count map:
	TArray<FMissedChunk> ChunkMissArray;
	for (auto& CacheMiss : CacheMissCount)
	{
		FMissedChunk MissedChunk =
		{
			CacheMiss.Key.SoundWaveName,
			CacheMiss.Key.ChunkIndex,
			CacheMiss.Value
		};

		ChunkMissArray.Add(MissedChunk);
	}

	ChunkMissArray.Sort(FCacheMissSortPredicate());

	FString TopChunkMissesLog = "Most Missed Chunks:\n";
	TopChunkMissesLog += TEXT("Name:\t, Index:\t, Miss Count:\n");
	for (FMissedChunk& MissedChunk : ChunkMissArray)
	{
		TopChunkMissesLog.Append(MissedChunk.SoundWaveName.ToString());
		TopChunkMissesLog.Append(TEXT("\t, "));
		TopChunkMissesLog.AppendInt(MissedChunk.ChunkIndex);
		TopChunkMissesLog.Append(TEXT("\t, "));
		TopChunkMissesLog.AppendInt(MissedChunk.MissCount);
		TopChunkMissesLog.Append(TEXT("\n"));
	}

	// If we are keeping the full cache miss buffer around, re-enqueue every cache miss we dequeued.
	// Note: This could be done more gracefully if TQueue had a move constructor.
	if (KeepCacheMissBufferOnFlushCVar)
	{
		while (BackupQueue.Dequeue(CacheMissInfo))
		{
			CacheMissQueue->Enqueue(CacheMissInfo);
		}
	}

	return TopChunkMissesLog + TEXT("\n") + ConcatenatedCacheMisses;
}

FAudioChunkCache::FCacheElement* FAudioChunkCache::FindElementForKey(const FChunkKey& InKey)
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

	const uint64 CacheOffset = GetCacheLookupIDForChunk(InKey);

	// If we have a known cache offset, access that chunk directly.
	if (CacheOffset != InvalidAudioStreamCacheLookupID)
	{
		check(CacheOffset < CachePool.Num());

		// Finally, sanity check that the key is still the same.
		if (CachePool[CacheOffset].Key == InKey)
		{
			return &CachePool[CacheOffset];
		}

		UE_LOG(LogAudioStreamCaching, Verbose, TEXT("Cache Miss for soundwave: %s. (Cache Lookup ID [%" UINT64_FMT "] currently stores chunk for Soundwave: %s"),
			*InKey.SoundWaveName.ToString(), CacheOffset, *CachePool[CacheOffset].Key.SoundWaveName.ToString());
	}

	if (EnableExhaustiveCacheSearchesCVar)
	{
		// Otherwise, linearly search the cache.
		if (SearchUsingChunkArrayCVar)
		{
			return LinearSearchChunkArrayForElement(InKey);
		}

		return LinearSearchCacheForElement(InKey);
	}

	return nullptr;
}

FAudioChunkCache::FCacheElement* FAudioChunkCache::LinearSearchCacheForElement(const FChunkKey& InKey)
{
	//Otherwise, linearly search the cache.
	FCacheElement* CurrentElement = MostRecentElement;

	// In debuggable situations, we breadcrumb how far down the cache the cache we were.
	int32 ElementPosition = 0;

	while (CurrentElement != nullptr)
	{
		if (InKey == CurrentElement->Key)
		{

#if DEBUG_STREAM_CACHE
			float& CMA = CurrentElement->DebugInfo.AverageLocationInCacheWhenNeeded;
			CMA += ((ElementPosition - CMA) / (CurrentElement->DebugInfo.NumTimesTouched + 1));
#endif
			UE_LOGF(LogAudioStreamCaching, Display, "Found element in cache using linear search (LRU)");
			return CurrentElement;
		}
		else
		{
			CurrentElement = CurrentElement->LessRecentElement;

			ElementPosition++;

			if (CurrentElement && ElementPosition >= ChunksInUse)
			{
				UE_LOGF(LogAudioStreamCaching, Warning, "Possible cycle in our LRU cache list. Please check to ensure any place FCacheElement::MoreRecentElement or FCacheElement::LessRecentElement is changed is locked by CacheMutationCriticalSection.");
				return nullptr;
			}
		}
	}

	return CurrentElement;
}

FAudioChunkCache::FCacheElement* FAudioChunkCache::LinearSearchChunkArrayForElement(const FChunkKey& InKey)
{
	for (int i = 0; i < ChunksInUse; ++i)
	{
		if (InKey == CachePool[i].Key)
		{
			UE_LOGF(LogAudioStreamCaching, Display, "Found element in cache using linear search (Chunk Array)");
			return &CachePool[i];
		}
	}

	return nullptr;
}

void FAudioChunkCache::TouchElement(FCacheElement* InElement)
{
	checkSlow(InElement);

	// Check to ensure we do not have any cycles in our list.
	// If this first check is hit, try to ensure that EvictLeastRecent chunk isn't evicting the top two chunks.
	check(MostRecentElement == nullptr || MostRecentElement != LeastRecentElement);
	check(InElement->LessRecentElement != InElement);

	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

	// If this is already the most recent element, we don't need to do anything.
	if (InElement == MostRecentElement)
	{
		return;
	}

	// If this was previously the least recent chunk, update LeastRecentElement.
	if (LeastRecentElement == InElement)
	{
		LeastRecentElement = InElement->MoreRecentElement;
	}

	FCacheElement* PreviousLessRecent = InElement->LessRecentElement;
	FCacheElement* PreviousMoreRecent = InElement->MoreRecentElement;
	FCacheElement* PreviousMostRecent = MostRecentElement;

	check(PreviousMostRecent != InElement);

	// Move this element to the top:
	MostRecentElement = InElement;
	InElement->MoreRecentElement = nullptr;
	InElement->LessRecentElement = PreviousMostRecent;

	if (PreviousMostRecent != nullptr)
	{
		PreviousMostRecent->MoreRecentElement = InElement;
	}

	if (PreviousLessRecent == PreviousMoreRecent)
	{
		return;
	}
	else
	{
		// Link InElement's previous neighbors together:
		if (PreviousLessRecent != nullptr)
		{
			PreviousLessRecent->MoreRecentElement = PreviousMoreRecent;
		}

		if (PreviousMoreRecent != nullptr)
		{
			PreviousMoreRecent->LessRecentElement = PreviousLessRecent;
		}
	}
}

bool FAudioChunkCache::ShouldAddNewChunk() const
{
	return (ChunksInUse < CachePool.Num()) && (GetCurrentMemoryUsageBytes() < MemoryLimitBytes);
}

FAudioChunkCache::FCacheElement* FAudioChunkCache::InsertChunk(const FChunkKey& InKey, const TSharedRef<const FSoundWaveData>& InSoundWavePtr)
{
	FCacheElement* CacheElement = nullptr;

	{
		FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

		if (ShouldAddNewChunk())
		{
			// We haven't filled up the pool yet, so we don't need to evict anything.
			CacheElement = &CachePool[ChunksInUse];
			CacheElement->CacheLookupID = ChunksInUse;
			ChunksInUse++;
		}
		else
		{
			static bool bLoggedCacheSaturated = false;
			if (!bLoggedCacheSaturated)
			{
				UE_LOGF(LogAudioStreamCaching, Display, "Audio Stream Cache: Using %d of %d chunks..", ChunksInUse, CachePool.Num());
				bLoggedCacheSaturated = true;
			}

			// The pools filled, so we're going to need to evict.
			CacheElement = EvictLeastRecentChunk();

			// If we blew the cache, it might be because we have too many loads in flight. Here we attempt to find a load in flight for an unreferenced chunk:
			if (BlockForPendingLoadOnCacheOverflowCVar && !CacheElement)
			{
				UE_LOGF(LogAudioStreamCaching, Warning, "Failed to find an available chunk slot in the audio streaming manager. Finding a load in flight for an unreferenced chunk and cancelling it.");
				CacheElement = EvictLeastRecentChunk(true);
			}

			if (!CacheElement)
			{
				UE_LOGF(LogAudioStreamCaching, Display, "Failed to find an available chunk slot in the audio streaming manager, likely because the cache was blown.");
				return nullptr;
			}
		}

		UE_CLOGF(CacheElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Inserting chunk index %d, ObjectKey=%ls",
			*InSoundWavePtr->GetFName().ToString(), InKey.ChunkIndex, *WriteToString<40>(InKey.ObjectKey));
		
		CacheElement->bIsLoaded = false;
		CacheElement->Key = InKey;
		CacheElement->SoundWaveWeakPtr = InSoundWavePtr;
		TouchElement(CacheElement);
#if DEBUG_STREAM_CACHE
		CacheElement->DebugInfo.Format = InSoundWavePtr->GetRuntimeFormat();
		const FStreamedAudioChunk& Chunk=InSoundWavePtr->GetChunk(InKey.ChunkIndex);
		const float SampleRate = InSoundWavePtr->GetSampleRate();
		const int32 NumChunks = InSoundWavePtr->GetNumChunks();
		const uint32 OffsetInFrames = Chunk.SeekOffsetInAudioFrames;
		CacheElement->DebugInfo.OffsetInSeconds = OffsetInFrames != INDEX_NONE ? OffsetInFrames / SampleRate : -1.f;
		CacheElement->DebugInfo.NumTotalChunks = NumChunks - 1; //(exclude chunk 0)
#endif //DEBUG_STREAM_CACHE		

		// If we've got multiple chunks, we can not cache the least recent chunk
		// without worrying about a circular dependency.
		if (LeastRecentElement == nullptr && ChunksInUse > 1)
		{
			SetUpLeastRecentChunk();
		}
	}

	SetCacheLookupIDForChunk(InKey, CacheElement->CacheLookupID);
	return CacheElement;
}

void FAudioChunkCache::SetUpLeastRecentChunk()
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

	FCacheElement* CacheElement = MostRecentElement;
	while (CacheElement->LessRecentElement != nullptr)
	{
		CacheElement = CacheElement->LessRecentElement;
	}

	LeastRecentElement = CacheElement;
}

FAudioChunkCache::FCacheElement* FAudioChunkCache::EvictLeastRecentChunk(bool bBlockForPendingLoads /* = false */)
{
	FCacheElement* CacheElement = LeastRecentElement;

	if (!CacheElement)
	{
		// This can happen if the MemoryLimitBytes is 0, prevting LeastRecentElement from being set to a valid element.
		return nullptr;
	}

	// If the least recent chunk is evictable, evict it.
	bool bIsChunkEvictable = CacheElement->CanEvictChunk();
	bool bIsChunkLoadingButUnreferenced = (CacheElement->IsLoadInProgress() && !CacheElement->IsInUse());

	if (bIsChunkEvictable)
	{
		UE_CLOGF(CacheElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Evicting ununsed LRU chunk index %d, ObjectKey=%ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), CacheElement->Key.ChunkIndex, *WriteToString<40>(CacheElement->Key.ObjectKey));

		check(CacheElement->MoreRecentElement != nullptr);
		check(CacheElement->LessRecentElement == nullptr);

		FCacheElement* NewLeastRecentElement = CacheElement->MoreRecentElement;
		check(NewLeastRecentElement);

		LeastRecentElement = NewLeastRecentElement;
	}
	else if (bBlockForPendingLoads && bIsChunkLoadingButUnreferenced)
	{
		UE_CLOGF(CacheElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Evicting ununsed loading chunk index %d, ObjectKey=%ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), CacheElement->Key.ChunkIndex, *WriteToString<40>(CacheElement->Key.ObjectKey));
		CacheElement->WaitForAsyncLoadCompletion(true);

		FCacheElement* NewLeastRecentElement = CacheElement->MoreRecentElement;
		check(NewLeastRecentElement);

		LeastRecentElement = NewLeastRecentElement;
	}
	else
	{
		// We should never hit this code path unless we have at least two chunks active.
		check(MostRecentElement && MostRecentElement->LessRecentElement);

		// In order to avoid cycles, we always leave at least two chunks in the cache.
		const FCacheElement* ElementToStopAt = MostRecentElement->LessRecentElement;

		// Otherwise, we need to crawl up the cache from least recent used to most to find a chunk that is not in use:
		while (CacheElement && CacheElement != ElementToStopAt)
		{
			// If the least recent chunk is evictable, evict it.
			bIsChunkEvictable = CacheElement->CanEvictChunk();
			bIsChunkLoadingButUnreferenced = (CacheElement->IsLoadInProgress() && !CacheElement->IsInUse());

			if (bIsChunkEvictable)
			{
				// Link the two neighboring chunks:
				if (CacheElement->MoreRecentElement)
				{
					CacheElement->MoreRecentElement->LessRecentElement = CacheElement->LessRecentElement;
				}

				// If we ever hit this while loop it means that CacheElement is not the least recently used element.
				check(CacheElement->LessRecentElement);
				CacheElement->LessRecentElement->MoreRecentElement = CacheElement->MoreRecentElement;
				break;
			}
			else if (bBlockForPendingLoads && bIsChunkLoadingButUnreferenced)
			{
				CacheElement->WaitForAsyncLoadCompletion(true);

				// Link the two neighboring chunks:
				if (CacheElement->MoreRecentElement)
				{
					CacheElement->MoreRecentElement->LessRecentElement = CacheElement->LessRecentElement;
				}

				// If we ever hit this while loop it means that CacheElement is not the least recently used element.
				check(CacheElement->LessRecentElement);
				CacheElement->LessRecentElement->MoreRecentElement = CacheElement->MoreRecentElement;
				break;
			}
			else
			{
				CacheElement = CacheElement->MoreRecentElement;
			}
		}

		// If we ever hit this, it means that we couldn't find any cache elements that aren't in use.
		if (!CacheElement || CacheElement == ElementToStopAt)
		{
			UE_LOGF(LogAudioStreamCaching, Warning, "Cache blown! Please increase the cache size (currently %llu bytes) or load less audio.", ReportCacheSize());
			return nullptr;
		}

#if !UE_BUILD_SHIPPING
		UE_CLOGF(CacheElement->bWantsExtraLogging, LogAudioStreamCaching, Display, "[%ls] - Evicting unused chunk index %d, ObjectKey=%ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), CacheElement->Key.ChunkIndex, *WriteToString<40>(CacheElement->Key.ObjectKey));
#endif
	}

#if DEBUG_STREAM_CACHE
	// Reset debug information:
	CacheElement->DebugInfo.Reset();
#endif

	return CacheElement;
}



static FAutoConsoleTaskPriority CPrio_ClearAudioChunkCacheReadRequest(
	TEXT("TaskGraph.TaskPriorities.ClearAudioChunkCacheReadRequest"),
	TEXT("Task and thread priority for an async task that clears FCacheElement::ReadRequest"),
	ENamedThreads::BackgroundThreadPriority, // if we have background priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::NormalTaskPriority // if we don't have background threads, then use normal priority threads at normal task priority instead
);

class FClearAudioChunkCacheReadRequestTask
{
	IBulkDataIORequest* ReadRequest;

public:
	FORCEINLINE FClearAudioChunkCacheReadRequestTask(IBulkDataIORequest* InReadRequest)
		: ReadRequest(InReadRequest)
	{
	}
	static FORCEINLINE TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FClearAudioChunkCacheReadRequestTask, STATGROUP_TaskGraphTasks);
	}
	static FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_ClearAudioChunkCacheReadRequest.Get();
	}
	FORCEINLINE static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::FireAndForget;
	}
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (ReadRequest)
		{
			ReadRequest->WaitCompletion();
			delete ReadRequest;
			ReadRequest = nullptr;
		}
	}
};

void FAudioChunkCache::KickOffAsyncLoad(FCacheElement* CacheElement, const FChunkKey& InKey, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted, ENamedThreads::Type CallbackThread, bool bNeededForPlayback)
{
	FStreamedAudioChunk* Chunk = CacheElement->GetChunk(InKey.ChunkIndex);

	if (nullptr == Chunk)
	{
		UE_LOGF(LogAudioStreamCaching, Error, "Failed to kick off async load of chunk %d in soundwave \"%ls\". Could not find chunk.", InKey.ChunkIndex, *InKey.SoundWaveName.ToString());
		return;
	}

	int32 ChunkDataSize = Chunk->AudioDataSize;

	EAsyncIOPriorityAndFlags AsyncIOPriority = GetAsyncPriorityForChunk(InKey, bNeededForPlayback);
	UE_CLOGF(CacheElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - KickOffAsyncLoad pri %d outstanding %d chunk index %d, ObjectKey=%ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), (uint32)AsyncIOPriority, NumberOfLoadsInFlight.GetValue(), CacheElement->Key.ChunkIndex, *WriteToString<40>(CacheElement->Key.ObjectKey));

	*MemoryCounterBytes -= CacheElement->ChunkDataSize;

	{
		LLM_SCOPE(ELLMTag::AudioStreamCacheCompressedData);

		// Reallocate our chunk data This allows us to shrink if possible.
		CacheElement->ChunkData = (uint8*)FMemory::Realloc(CacheElement->ChunkData, Chunk->AudioDataSize);
		CacheElement->ChunkDataSize = Chunk->AudioDataSize;
	}

	*MemoryCounterBytes += CacheElement->ChunkDataSize;

#if DEBUG_STREAM_CACHE
	CacheElement->DebugInfo.NumTotalChunks = CacheElement->GetNumChunks() - 1;
	CacheElement->UpdateDebugInfoLoadingBehavior();
#endif

	// In editor, we retrieve from the DDC. In non-editor situations, we read the chunk async from the pak file.
#if WITH_EDITORONLY_DATA
	if (Chunk->DerivedDataKey.IsEmpty() == false)
	{
		CacheElement->ChunkDataSize = ChunkDataSize;

		INC_DWORD_STAT_BY(STAT_AudioMemorySize, ChunkDataSize);
		INC_DWORD_STAT_BY(STAT_AudioMemory, ChunkDataSize);

		if (CacheElement->DDCTask.IsValid())
		{
			UE_CLOGF(!CacheElement->DDCTask->IsDone(), LogAudioStreamCaching, Display, "DDC work was not finished for a requested audio streaming chunk slot berfore reuse; This may cause a hitch.");
			CacheElement->DDCTask->EnsureCompletion();
		}

#if DEBUG_STREAM_CACHE
		CacheElement->DebugInfo.TimeLoadStartedCycles = FPlatformTime::Cycles64();
#endif

		TFunction<void(bool)> OnLoadComplete = [OnLoadCompleted, CallbackThread, CacheElement, InKey, ChunkDataSize](bool bRequestFailed)
		{
			// Populate key and DataSize. The async read request was set up to write directly into CacheElement->ChunkData.
			// The following condition should always be true and there should be no need
			// to overwrite the Key as it can cause race condition between the callback thread
			// and other threads trying to search for elements by key.
			
			// If this ensure is tripped for some reason, we must find the root cause, not remove the ensure.
			ensure(CacheElement->Key == InKey);
			// This can be removed later once we're sure the ensure is never tripped
			// For now, avoid overwriting when both values are the same to avoid a race condition.
			if (!(CacheElement->Key == InKey))
			{
				CacheElement->Key = InKey;
			}

			CacheElement->ChunkDataSize = ChunkDataSize;
			CacheElement->bIsLoaded = true;
			if (bRequestFailed)
			{
				FMemory::Memzero(CacheElement->ChunkData, CacheElement->ChunkDataSize);
				UE_LOGF(LogAudio, Warning, "FAudioChunkCache::KickOffAsyncLoad -> DDCTask.OnLoadComplete: Request Failed. ChunkIdx: %d; SoundWave: %ls", InKey.ChunkIndex, *InKey.SoundWaveName.ToString());
			}
			
#if DEBUG_STREAM_CACHE
			CacheElement->DebugInfo.TimeLoadEndedCycles = FPlatformTime::Cycles64();
			CacheElement->DebugInfo.TimeToLoadMs = FPlatformTime::ToMilliseconds64(CacheElement->DebugInfo.TimeLoadEndedCycles - CacheElement->DebugInfo.TimeLoadStartedCycles);
#endif
			EAudioChunkLoadResult ChunkLoadResult = bRequestFailed ? EAudioChunkLoadResult::Interrupted : EAudioChunkLoadResult::Completed;
			ExecuteOnLoadCompleteCallback(ChunkLoadResult, OnLoadCompleted, CallbackThread);
		};

		NumberOfLoadsInFlight.Increment();
		AddLoadCounters(AsyncIOPriority, 1);

		CacheElement->DDCTask.Reset(new FAsyncStreamDerivedChunkTask(
			Chunk->DerivedDataKey,
			CacheElement->ChunkData,
			ChunkDataSize,
			&NumberOfLoadsInFlight,
			MoveTemp(OnLoadComplete)
		));

		// This task may perform a long synchronous DDC request. Using DoNotRunInsideBusyWait prevents potentially delaying foreground tasks.
		CacheElement->DDCTask->StartBackgroundTask(GThreadPool, EQueuedWorkPriority::Normal, EQueuedWorkFlags::DoNotRunInsideBusyWait);
	}
	else
#endif // #if WITH_EDITORONLY_DATA
	{
		if (CacheElement->IsLoadInProgress())
		{
			// Update priority if it's already pending...		
			if (CacheElement->Read.IsPending() && AsyncIOPriority > CacheElement->Priority)
			{
				CacheElement->Read.UpdatePriority(AsyncIOPriority);
				CacheElement->Priority = AsyncIOPriority;
			}
			UE_CLOGF(CacheElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Waiting for async load while kicking async load, chunk %d, ObjectKey=%ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), CacheElement->Key.ChunkIndex, *WriteToString<40>(CacheElement->Key.ObjectKey));
			CacheElement->WaitForAsyncLoadCompletion(true);
		}

		// Sanity check our bulk data against our currently allocated chunk size in the cache.
		const int32 ChunkBulkDataSize = Chunk->BulkData.GetBulkDataSize();
		check(ChunkDataSize <= ChunkBulkDataSize);
		check(((uint32)ChunkDataSize) <= CacheElement->ChunkDataSize);

		// If we ever want to eliminate zero-padding in chunks, that could be verified here:
		//ensureAlwaysMsgf(AudioChunkSize == ChunkBulkDataSize, TEXT("For memory load on demand, we do not zero-pad to page sizes."));

		NumberOfLoadsInFlight.Increment();
		AddLoadCounters(AsyncIOPriority, 1);

		FBulkDataRequest::FCompletionCallback AsyncFileCallBack = [this, OnLoadCompleted, CacheElement, InKey, ChunkDataSize, CallbackThread, AsyncIOPriority](FBulkDataRequest::EStatus Status) 
		{
			const bool bWasCancelled = Status == FBulkDataRequest::EStatus::Cancelled;
			// Diagnostic logging.
			const bool bMatched = CacheElement->Key == InKey;
			UE_CLOGF(CacheElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - AsyncReadComplete index %d, ObjectKey=%ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), CacheElement->Key.ChunkIndex, *WriteToString<40>(CacheElement->Key.ObjectKey));
			UE_CLOGF(CacheElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Cancelled=%d, Size=%u, InKey %ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), bWasCancelled, ChunkDataSize, bMatched ? TEXT("matched") : TEXT("DID NOT MATCH!"));
			UE_CLOGF(CacheElement->WantsFilteredLogging() && !bMatched, LogAudioTiming, Verbose, "[%ls] - InKey %ls / %d / %ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), *WriteToString<128>(InKey.SoundWaveName), InKey.ChunkIndex, *WriteToString<40>(InKey.ObjectKey));

			// Populate key and DataSize. The async read request was set up to write directly into CacheElement->ChunkData.
			CacheElement->Key = InKey;
			CacheElement->ChunkDataSize = ChunkDataSize;
			CacheElement->bIsLoaded = Status == FBulkDataRequest::EStatus::Ok;

			if (bWasCancelled)
			{
				FMemory::Memzero(CacheElement->ChunkData, CacheElement->ChunkDataSize);
				UE_LOGF(LogAudio, Warning, "FAudioChunkCache::KickOffAsyncLoad -> AsyncFileCallBack: Request Cancelled. ChunkIdx: %d; SoundWave: %ls", InKey.ChunkIndex, *InKey.SoundWaveName.ToString());
			}
			
#if DEBUG_STREAM_CACHE
			CacheElement->DebugInfo.TimeLoadEndedCycles = FPlatformTime::Cycles64();
			CacheElement->DebugInfo.TimeToLoadMs = FPlatformTime::ToMilliseconds64(CacheElement->DebugInfo.TimeLoadEndedCycles - CacheElement->DebugInfo.TimeLoadStartedCycles);
#endif
			const EAudioChunkLoadResult LoadResult = bWasCancelled ? EAudioChunkLoadResult::Interrupted : EAudioChunkLoadResult::Completed;
			ExecuteOnLoadCompleteCallback(LoadResult, OnLoadCompleted, CallbackThread);

			NumberOfLoadsInFlight.Decrement();
			AddLoadCounters(AsyncIOPriority, -1);
		};

#if DEBUG_STREAM_CACHE
		CacheElement->DebugInfo.TimeLoadStartedCycles = FPlatformTime::Cycles64();
#endif

		CacheElement->Read = {};
		if (Chunk->BulkData.IsBulkDataLoaded())
		{
			UE_CLOGF(CacheElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "[%ls] - Chunk %d has bulk data loaded, , ObjectKey=%ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), CacheElement->Key.ChunkIndex, *WriteToString<40>(CacheElement->Key.ObjectKey));
			// If this chunk has been inlined and loaded, move out the data into our newly allocated block.
			const FBulkDataBuffer<uint8> ChunkMemory = Chunk->MoveOutAsBuffer();
			
			// Copy and delete to be sure we pay back the LLM and use our newly allocated version.
			check(CacheElement->ChunkDataSize <= ChunkMemory.GetView().Num());
			FMemory::Memcpy(CacheElement->ChunkData, ChunkMemory.GetView().GetData(), ChunkMemory.GetView().Num());

#if DEBUG_STREAM_CACHE
			UE_LOGF(LogAudioStreamCaching, Verbose, "Loading Inlined Chunk: %ls, %d, TimeToLoad=%2.2fms", *InKey.SoundWaveName.ToString(),
				InKey.ChunkIndex, FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - CacheElement->DebugInfo.TimeLoadStartedCycles));

			CacheElement->DebugInfo.bWasLoadedFromInlineChunk = true;
			UE_LOGF(LogAudioStreamCaching, VeryVerbose, "COPY+DISCARD %ls - Bulk=0x%p", *InKey.SoundWaveName.ToString(), &Chunk->BulkData);
#endif //DEBUG_STREAM_CACHE
			
			// Fire the callback (this will mark it load completed etc).
			AsyncFileCallBack(FBulkDataRequest::EStatus::Ok);
		}
		else
		{
			UE_LOGF(LogAudioStreamCaching, VeryVerbose, "DISK %ls - Bulk=0x%p", *InKey.SoundWaveName.ToString(), &Chunk->BulkData)

#if DEBUG_STREAM_CACHE
			CacheElement->DebugInfo.bWasInlinedButUnloaded = Chunk->BulkData.IsInlined() || Chunk->BulkData.GetBulkDataFlags() & BULKDATA_ForceInlinePayload;
			UE_CLOGF(CacheElement->DebugInfo.bWasInlinedButUnloaded,LogAudioStreamCaching, Log, "IO LOAD FOR INLINE %ls - Bulk=0x%p", *InKey.SoundWaveName.ToString(), &Chunk->BulkData);
#endif //DEBUG_STREAM_CACHE
			
			UE_LOGF(LogAudioStreamCaching, Verbose, "Loading Chunk: %ls, %d", *InKey.SoundWaveName.ToString(), InKey.ChunkIndex);

#if !UE_BUILD_SHIPPING
			UE_CLOGF(CacheElement->bWantsExtraLogging, LogAudioStreamCaching, Display, "[%ls] - CreateStreamingRequest: chunk %d, ObjectKey=%ls", *WriteToString<128>(CacheElement->Key.SoundWaveName), CacheElement->Key.ChunkIndex, *WriteToString<40>(CacheElement->Key.ObjectKey));
#endif
			
			TRACE_IOSTORE_METADATA_SCOPE_TAG(InKey.SoundWaveName);
			FBulkDataBatchRequest::FBatchBuilder Builder = FBulkDataBatchRequest::NewBatch(1);
			FIoBuffer Buffer(FIoBuffer::Wrap, CacheElement->ChunkData,CacheElement->ChunkDataSize);
			Builder.Read(Chunk->BulkData, 0, CacheElement->ChunkDataSize, AsyncIOPriority | AIOP_FLAG_DONTCACHE, Buffer);
			Builder.Issue(MoveTemp(AsyncFileCallBack), CacheElement->Read);
			CacheElement->Priority = AsyncIOPriority;
		}
	}
}


EAsyncIOPriorityAndFlags FAudioChunkCache::GetAsyncPriorityForChunk(const FChunkKey& InKey, bool bNeededForPlayback)
{
	if (bNeededForPlayback)
	{
		return ToIoPriority(PlaybackRequestPriorityCVar);
	}
	return ToIoPriority(ReadRequestPriorityCVar);
}	

void FAudioChunkCache::ExecuteOnLoadCompleteCallback(EAudioChunkLoadResult Result, const TFunction<void(EAudioChunkLoadResult)>& OnLoadCompleted, const ENamedThreads::Type& CallbackThread)
{
	if (CallbackThread == ENamedThreads::AnyThread)
	{
		OnLoadCompleted(Result);
	}
	else
	{
		// Dispatch an async notify.
		AsyncTask(CallbackThread, [OnLoadCompleted, Result]()
		{
			OnLoadCompleted(Result);
		});
	}
}

bool FAudioChunkCache::DoesKeyContainValidChunkIndex(const FChunkKey& InKey, const FSoundWaveData& InSoundWaveData)
{
	return InKey.ChunkIndex < TNumericLimits<uint32>::Max() && ((int32)InKey.ChunkIndex) < InSoundWaveData.GetNumChunks();
}

uint64 FAudioChunkCache::GetCacheLookupIDForChunk(const FChunkKey& InChunkKey) const
{
	FScopeLock Lock(&CacheMutationCriticalSection.Get());
	const uint64* ID = CacheLookupIdMap.Find(InChunkKey);

	if (ID)
	{
		return *ID;
	}
	else
	{
		return InvalidAudioStreamCacheLookupID;
	}
}

void FAudioChunkCache::SetCacheLookupIDForChunk(const FChunkKey& InChunkKey, uint64 InCacheLookupID)
{
	FScopeLock Lock(&CacheMutationCriticalSection.Get());
	uint64& ID = CacheLookupIdMap.FindOrAdd(InChunkKey);
	ID = InCacheLookupID;
}


void FAudioChunkCache::HandleStarvation(
	const TSharedRef<const FSoundWaveData>& InSoundWaveData, const uint32 ChunkIndex, const uint32 InBytesProduced, const uint32 InBytesExpected)
{
	const FChunkKey Key(
		InSoundWaveData
		, ChunkIndex
#if WITH_EDITOR
		, (uint32)InSoundWaveData->GetCurrentChunkRevision()
#endif
	);
	if (FCacheElement* FoundElement = FindElementForKey(Key))
	{
		const float PercentProduced = InBytesExpected > 0.f ? InBytesProduced / (float)InBytesExpected : 0.f;
		const float Starvometer = 1.f - PercentProduced;
		
		// Stream is playing, and we're starving, crank priority of read
		static constexpr EAsyncIOPriorityAndFlags StarvingPriority = AIOP_MAX;
		if (FoundElement->Read.IsPending() && FoundElement->Priority != StarvingPriority)
		{
#if DEBUG_STREAM_CACHE
			const float TimeStartedSecs = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64()-FoundElement->DebugInfo.TimeLoadStartedCycles);
			UE_CLOGF(FoundElement->WantsFilteredLogging(), LogAudioTiming, Verbose, "FAudioChunkCache::HandleStarvation Wave:%ls, Chunk:%d, LoadRequested=-%.2f (secs), OldPriority=%ls, NewPriority=%ls, Starvometer=%2.f",
				*InSoundWaveData->GetFName().ToString(), ChunkIndex, TimeStartedSecs, LexToString(FoundElement->Priority), LexToString(StarvingPriority), Starvometer);
#endif //DEBUG_STREAM_CACHE

			FoundElement->Read.UpdatePriority(AIOP_MAX);
			FoundElement->Priority = StarvingPriority;
		}
		
		return;
	}
	UE_LOG(LogAudioTiming, Verbose, TEXT("Decoder Starving waiting for Wave:%s, Chunk:%d (not in cache!)"), *InSoundWaveData->GetFName().ToString(), ChunkIndex)
}

int32 FCachedAudioStreamingManager::RenderStatAudioStreaming(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	Canvas->DrawShadowedString(X, Y, TEXT("Stream Caches:"), UEngine::GetSmallFont(), FLinearColor::White);
	Y += 12;

	int32 CacheIndex = 0;
	int32 Height = Y;
	for (const FAudioChunkCache& Cache : CacheArray)
	{
		FString CacheTitle = *FString::Printf(TEXT("Cache %d"), CacheIndex);
		Canvas->DrawShadowedString(X, Y, *CacheTitle, UEngine::GetSmallFont(), FLinearColor::White);
		Y += 12;

		TPair<int, int> Size = Cache.DebugDisplay(World, Viewport, Canvas, X, Y, ViewLocation, ViewRotation);

		// Separate caches are laid out horizontally across the screen, so the total height is equal to our tallest cache panel:
		X += Size.Key;
		Height = FMath::Max(Height, Size.Value);
	}

	return Y + Height;
}

FString FCachedAudioStreamingManager::GenerateMemoryReport()
{
	FString OutputString;
	for (FAudioChunkCache& Cache : CacheArray)
	{
		OutputString += Cache.DebugPrint();
	}

	return OutputString;
}

void FCachedAudioStreamingManager::SetProfilingMode(bool bEnabled)
{
	if (bEnabled)
	{
		for (FAudioChunkCache& Cache : CacheArray)
		{
			Cache.BeginLoggingCacheMisses();
		}
	}
	else
	{
		for (FAudioChunkCache& Cache : CacheArray)
		{
			Cache.StopLoggingCacheMisses();
		}
	}
}

uint64 FCachedAudioStreamingManager::TrimMemory(uint64 NumBytesToFree)
{
	uint64 NumBytesLeftToFree = NumBytesToFree;

	// TODO: When we support multiple caches, it's probably best to do this in reverse,
	// since the caches are sorted from shortest sounds to longest.
	// Freeing longer chunks will get us bigger gains and (presumably) have lower churn.
	for (FAudioChunkCache& Cache : CacheArray)
	{
		const uint64 NumBytesFreed = Cache.TrimMemory(NumBytesLeftToFree, false);

		// NumBytesFreed could potentially be more than what we requested to free (since we delete whole chunks at once).
		NumBytesLeftToFree -= FMath::Min(NumBytesFreed, NumBytesLeftToFree);

		// If we've freed all the memory we needed to, exit.
		if (NumBytesLeftToFree == 0)
		{
			break;
		}
	}

	check(NumBytesLeftToFree <= NumBytesToFree);
	const uint64 TotalBytesFreed = NumBytesToFree - NumBytesLeftToFree;

	UE_LOGF(LogAudioStreamCaching, Display, "Call to IAudioStreamingManager::TrimMemory successfully freed %llu of the requested %llu bytes.", TotalBytesFreed, NumBytesToFree);
	return TotalBytesFreed;
}

#include "CanvasTypes.h"
#include "Engine/Font.h"

TPair<int, int> FAudioChunkCache::DebugDisplayLegacy(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) const
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

	// Color scheme:
	static constexpr float ColorMax = 256.0f;


	// Chunk color for a single retainer.
	const FLinearColor RetainChunkColor(44.0f / ColorMax, 207.0f / ColorMax, 47 / ColorMax);

	// Chunk color we lerp to as more retainers are added for a chunk.
	const FLinearColor TotalMassRetainChunkColor(204 / ColorMax, 126 / ColorMax, 43 / ColorMax);

	// A chunk that's loaded but not retained.
	const FLinearColor LoadedChunkColor(47 / ColorMax, 44 / ColorMax, 207 / ColorMax);

	// A chunk that's been trimmed by TrimMemory.
	const FLinearColor TrimmedChunkColor(204 / ColorMax, 46 / ColorMax, 43 / ColorMax);

	// In editor builds, this is a chunk that was built in a previous version of the cook quality settings.
	const FLinearColor StaleChunkColor(143 / ColorMax, 73 / ColorMax, 70 / ColorMax);

	// A chunk that currently has an async load in flight.
	const FLinearColor CurrentlyLoadingChunkColor = FLinearColor::Yellow;

	TArray<TPair<FCacheElement*, int32>> SortedCacheElements;

	int32 SortMethod = CDSM_RecentlyUsed;
#if DEBUG_STREAM_CACHE
	SortMethod = DebugCacheDisplaySortMethod;
#endif

	{
		FCacheElement* CurrentElement = MostRecentElement;
		for (int32 Index = 0; CurrentElement; Index++, CurrentElement = CurrentElement->LessRecentElement)
		{
#if DEBUG_STREAM_CACHE
			if (SortMethod == CDSM_LoadTime ||
				SortMethod == CDSM_LoadStartTime)
			{
				// If we care about load time, don't display stuff that didn't have a load time.
				if (CurrentElement->DebugInfo.bWasLoadedFromInlineChunk)
				{
					continue;
				}
			}
#endif
			SortedCacheElements.Add({CurrentElement, Index});
		}
	}


	switch (SortMethod)
	{
	case CDSM_RecentlyUsed: break; // already sorted this way.
#if DEBUG_STREAM_CACHE
	case CDSM_LoadTime:
		{
			Algo::SortBy(SortedCacheElements, [](TPair<FCacheElement*, int32> ElementPair) { return ElementPair.Key->DebugInfo.TimeToLoadMs; }, TGreater<float>() );
			break;
		}
	case CDSM_AccessCount:
		{
			Algo::SortBy(SortedCacheElements, [](TPair<FCacheElement*, int32> ElementPair) { return ElementPair.Key->DebugInfo.NumTimesTouched; }, TGreater<int32>() );
			break;
		}
	case CDSM_RetainingHandleCount:
		{
			Algo::SortBy(SortedCacheElements, [](TPair<FCacheElement*, int32> ElementPair) { return ElementPair.Key->NumConsumers.GetValue(); }, TGreater<int32>() );
			break;
		}
	case CDSM_LoadStartTime:
		{
			Algo::SortBy(SortedCacheElements, [](TPair<FCacheElement*, int32> ElementPair) { return ElementPair.Key->DebugInfo.TimeLoadStartedCycles; }, TGreater<uint64>() );
			break;
		}
#endif
	};

	// Trim to our display count
	if (SortedCacheElements.Num() > DebugMaxElementsDisplayCVar)
	{
		SortedCacheElements.SetNum(DebugMaxElementsDisplayCVar);
	}
	
	const int32 InitialX = X;
	const int32 InitialY = Y;

	FString NumElementsDetail = *FString::Printf(TEXT("Number of chunks loaded: %d of %d"), ChunksInUse, CachePool.Num());

	const int32 NumCacheOverflows = CacheOverflowCount.GetValue();
	FString CacheOverflowsDetail = *FString::Printf(TEXT("The cache has blown %d times)"), NumCacheOverflows);

	// Offset our number of elements loaded horizontally to the right next to the cache title:
	int32 CacheTitleOffsetX = 0;
	int32 CacheTitleOffsetY = 0;
	UEngine::GetSmallFont()->GetStringHeightAndWidth(TEXT("Cache XX "), CacheTitleOffsetY, CacheTitleOffsetX);

	// First pass: We run through and get a snap shot of the amount of memory currently in use.

	float ColorLerpAmount = 0.0f;
	const float ColorLerpStep = 0.04f;
	uint64 CurrentTime = FPlatformTime::Cycles64();

	Audio::FTabularRenderHelper Helper;
	Helper.AddCol(TEXT("Name"), 64);
	Helper.AddCol(TEXT("Chunk"), 10);
	Helper.AddCol(TEXT("Size(kb)"), 10);
	Helper.AddCol(TEXT("Loadtime(ms)"),20);
	Helper.AddCol(TEXT("Loaded at(sec)"),20);
	Helper.AddCol(TEXT("Load-Behavior"), 20);
	Helper.AddCol(TEXT("Format"), 8);
	Helper.AddCol(TEXT("Refs"), 6);
	Helper.AddCol(TEXT("Reqs"), 6);
	Helper.AddCol(TEXT("Offset(secs)"), 20);
	Helper.AddCol(TEXT("Inlined"),10);
	Helper.AddCol(TEXT("Wasinline"),10);
	Helper.AddCol(TEXT("Cachemiss"),10);
	Helper.AddCol(TEXT("Stale"), 10);
	Helper.AddCol(TEXT("Destroyed"),10);
	
	// More detailed info about individual chunks here:
	for (TPair<FCacheElement*, int32>& ElementPair : SortedCacheElements)
	{
		FCacheElement* CurrentElement = ElementPair.Key;
		int32 IndexInLRUList = ElementPair.Value;

		int32 NumTotalChunks = -1;
		int32 NumTimesTouched = -1;
		float TimeToLoad = -1.0;
		ESoundWaveLoadingBehavior LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
		bool bLoadingBehaviorExternallyOverriden = false;
		bool bWasCacheMiss = false;
		bool bIsStaleChunk = false;
		bool bWasLoadedInlined = false;
		bool bWasInlinedButUnloaded = false;
		bool bWaveDestroyed = false;
		float OffsetInSecs = 0;
		FName Format;
		double LoadedAt = 0;

#if DEBUG_STREAM_CACHE
		NumTotalChunks = CurrentElement->DebugInfo.NumTotalChunks;
		NumTimesTouched = CurrentElement->DebugInfo.NumTimesTouched;
		TimeToLoad = CurrentElement->DebugInfo.TimeToLoadMs;
		LoadingBehavior = CurrentElement->DebugInfo.LoadingBehavior;
		bLoadingBehaviorExternallyOverriden = CurrentElement->DebugInfo.bLoadingBehaviorExternallyOverriden;
		bWasCacheMiss = CurrentElement->DebugInfo.bWasCacheMiss;
		bWasLoadedInlined = CurrentElement->DebugInfo.bWasLoadedFromInlineChunk;
		bWasInlinedButUnloaded = CurrentElement->DebugInfo.bWasInlinedButUnloaded;
		bWaveDestroyed = CurrentElement->DebugInfo.bWaveDestroyed;
		OffsetInSecs = CurrentElement->DebugInfo.OffsetInSeconds;
		Format = CurrentElement->DebugInfo.Format;
		TimeToLoad = CurrentElement->DebugInfo.TimeToLoadMs;
		LoadedAt = -FPlatformTime::ToSeconds64(CurrentTime - CurrentElement->DebugInfo.TimeLoadStartedCycles);

		// we want the next entry to be relative to the first, so only the first entry goes backward in time and
		// it's easier to spot big load jumps
		if (SortMethod == CDSM_LoadStartTime)
		{
			CurrentTime = CurrentElement->DebugInfo.TimeLoadStartedCycles;
		}
#endif

#if WITH_EDITOR
		// TODO: Worry about whether the sound wave is alive here. In most editor cases this is ok because the soundwave will always be loaded, but this may not be the case in the future.
		bIsStaleChunk = CurrentElement->IsChunkStale();
#endif

		const bool bWasTrimmed = CurrentElement->ChunkDataSize == 0;

		// Since there's a lot of info here,
		// Subtly fading the chunk info to gray seems to help as a visual indicator of how far down on the list things are.
		ColorLerpAmount = FMath::Min(ColorLerpAmount + ColorLerpStep, 1.0f);
		FLinearColor TextColor;
		
		if (bIsStaleChunk)
		{
			TextColor = FLinearColor::LerpUsingHSV(StaleChunkColor, FLinearColor::Gray, ColorLerpAmount);
		}
		else
		{
			TextColor = FLinearColor::LerpUsingHSV(LoadedChunkColor, FLinearColor::Gray, ColorLerpAmount);
		}

		// If there's a load in flight, paint this element yellow.
		if (CurrentElement->IsLoadInProgress())
		{
			TextColor = FLinearColor::Yellow;
		}
		else if (CurrentElement->IsInUse())
		{
			// We slowly fade our text color based on how many refererences there are to this chunk.
			static const float MaxNumHandles = 12.0f;

			ColorLerpAmount = FMath::Min(CurrentElement->NumConsumers.GetValue() / MaxNumHandles, 1.0f);
			TextColor = FLinearColor::LerpUsingHSV(RetainChunkColor, TotalMassRetainChunkColor, ColorLerpAmount);
		}
		else if (bWasTrimmed)
		{
			TextColor = TrimmedChunkColor;
		}
#if !UE_BUILD_SHIPPING
		if (CurrentElement->bWantsExtraLogging)
		{
			TextColor = FLinearColor(1.0f, 0, 1.0f, 1.0f);
		}
#endif
		// Scale by load time.
		const float T = FMath::GetMappedRangeValueClamped( FVector2D(0.f, 100.f), {0.f,1.f}, TimeToLoad);
		FLinearColor LoadColor = FLinearColor::LerpUsingHSV(FColor::Green, FColor::Red, T);

		using namespace Audio;
		Helper.AddRow({
			{CurrentElement->Key.SoundWaveName.ToString(), TextColor},											// name
			{FString::Printf(TEXT("%d/%d"), CurrentElement->Key.ChunkIndex, NumTotalChunks), TextColor}, // chunk/count
			{FString::Printf(TEXT("%u"),  CurrentElement->ChunkDataSize >> 10),TextColor},			    // size (kb)
			{FString::Printf(TEXT("%.2f"), TimeToLoad), LoadColor},								        // load time (ms)
			{FString::Printf(TEXT("%.2f"), LoadedAt), LoadColor},
			{ShortEnumString(EnumToString(LoadingBehavior)), TextColor },									    // load behavior
			{Format.ToString(), TextColor },																    // format
			{FString::Printf(TEXT("%d"), CurrentElement->NumConsumers.GetValue()), TextColor},		    // hard refs
			{FString::Printf(TEXT("%d"), NumTimesTouched), TextColor},								    // requests
			{FString::Printf(TEXT("%.2f"), OffsetInSecs), TextColor},								    // offset(secs)
			{LexToString(bWasLoadedInlined), TextColor},													    // inlined?
			{LexToString(bWasCacheMiss), TextColor},														    // cache miss?
			{LexToString(bWasInlinedButUnloaded), TextColor},												    // inlinedButUnloaded?
			{LexToString(bIsStaleChunk), TextColor},														    // stale?
			{LexToString(bWaveDestroyed), TextColor},														    // destroyed?
		});
	}

	Helper.Draw(Canvas,X,Y);
	
	return TPair<int, int>(X - InitialX, Y - InitialY);
}

FString FAudioChunkCache::DebugPrint()
{
	FScopeLock ScopeLock(const_cast<FCriticalSection*>(&CacheMutationCriticalSection.Get()));

	FString OutputString;

	FString NumElementsDetail = *FString::Printf(TEXT("Number of chunks loaded: %d of %d"), ChunksInUse, CachePool.Num());
	FString NumCacheOverflows = *FString::Printf(TEXT("The cache has blown %d times"), CacheOverflowCount.GetValue());

	OutputString += NumElementsDetail + TEXT("\n");
	OutputString += NumCacheOverflows + TEXT("\n");

	// First pass: We run through and get a snap shot of the amount of memory currently in use.
	FCacheElement* CurrentElement = MostRecentElement;
	uint32 NumBytesCounter = 0;

	uint32 NumBytesRetained = 0;

	while (CurrentElement != nullptr)
	{
		// Note: this is potentially a stale value if we're in the middle of FCacheElement::KickOffAsyncLoad.
		NumBytesCounter += CurrentElement->ChunkDataSize;

		if (CurrentElement->IsInUse())
		{
			NumBytesRetained += CurrentElement->ChunkDataSize;
		}

		CurrentElement = CurrentElement->LessRecentElement;
	}

	// Num bytes in use should include Force Inline data!
	NumBytesCounter += *ForceInlineMemoryCounterBytes;

	// Num bytes should include feature data!
	NumBytesCounter += *FeatureMemoryCounterBytes;

	// Convert to megabytes and print the total size:
	const double NumMegabytesInUse = (double)NumBytesCounter / (1024 * 1024);
	const double NumMegabytesForceInline = (double)*ForceInlineMemoryCounterBytes / (1024 * 1024);
	const double NumMegabytesExternalFeatures = (double)*FeatureMemoryCounterBytes / (1024 * 1024);
	const double NumMegabytesRetained = (double)NumBytesRetained / (1024 * 1024);

	const double MaxCacheSizeMB = ((double)MemoryLimitBytes) / (1024 * 1024);
	const double PercentageOfCacheRetained = NumMegabytesRetained / MaxCacheSizeMB;
	const double PercentageOfCacheForceInlined = NumMegabytesForceInline / MaxCacheSizeMB;
	const double PercentageOfCacheExternalFeatures = NumMegabytesExternalFeatures / MaxCacheSizeMB;

	FString CacheMemoryHeader = *FString::Printf(TEXT("External Features:\t, Force Inline:\t, Retaining:\t, Loaded:\t, Max Potential Usage:\t, \n"));
	FString CacheMemoryUsage = *FString::Printf(TEXT("%.4f Megabytes (%.3f%% of total capacity)\t %.4f Megabytes (%.3f%% of total capacity)\t %.4f Megabytes (%.3f%% of total capacity)\t,  %.4f Megabytes (%" UINT64_FMT " bytes)\t, %.4f Megabytes\t, \n"), 
		NumMegabytesExternalFeatures, PercentageOfCacheExternalFeatures, NumMegabytesForceInline, PercentageOfCacheForceInlined, NumMegabytesRetained, PercentageOfCacheRetained, NumMegabytesInUse, MemoryCounterBytes->Load(), MaxCacheSizeMB);
	OutputString += CacheMemoryHeader + CacheMemoryUsage + TEXT("\n");

	// Second Pass: We're going to list the actual chunks in the cache.
	CurrentElement = MostRecentElement;
	int32 Index = 0;

	OutputString += TEXT("Index:\t, Size (KB):\t, Chunk:\t, Request Count:\t, Average Index:\t, Number of Handles Retaining Chunk:\t, Chunk Load Time:\t, Name: \t, LoadingBehavior: \t, Notes:\t, \n");

	// More detailed info about individual chunks here:
	while (CurrentElement != nullptr)
	{
		int32 NumTotalChunks = -1;
		int32 NumTimesTouched = -1;
		double TimeToLoad = -1.0;
		float AveragePlaceInCache = -1.0f;
		ESoundWaveLoadingBehavior LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
		bool bLoadingBehaviorExternallyOverriden = false;
		bool bWasCacheMiss = false;
		bool bIsStaleChunk = false;
		bool bWasLoadedInlined = false;
		bool bWasInlinedButUnloaded = false;
		bool bWaveDestroyed = false;

#if DEBUG_STREAM_CACHE
		NumTotalChunks = CurrentElement->DebugInfo.NumTotalChunks;
		NumTimesTouched = CurrentElement->DebugInfo.NumTimesTouched;
		TimeToLoad = CurrentElement->DebugInfo.TimeToLoadMs;
		AveragePlaceInCache = CurrentElement->DebugInfo.AverageLocationInCacheWhenNeeded;
		LoadingBehavior = CurrentElement->DebugInfo.LoadingBehavior;
		bLoadingBehaviorExternallyOverriden = CurrentElement->DebugInfo.bLoadingBehaviorExternallyOverriden;
		bWasCacheMiss = CurrentElement->DebugInfo.bWasCacheMiss;
		bWasLoadedInlined = CurrentElement->DebugInfo.bWasLoadedFromInlineChunk;
		bWasInlinedButUnloaded = CurrentElement->DebugInfo.bWasInlinedButUnloaded;
		bWaveDestroyed = CurrentElement->DebugInfo.bWaveDestroyed;
#endif

#if WITH_EDITOR
		// TODO: Worry about whether the sound wave is alive here. In most editor cases this is ok because the soundwave will always be loaded, but this may not be the case in the future.
		bIsStaleChunk = CurrentElement->IsChunkStale();
#endif

		const bool bWasTrimmed = CurrentElement->ChunkDataSize == 0;

		using namespace Audio;
		FString ElementInfo = *FString::Printf(TEXT("%4i.\t, %6.2f\t, %d of %d\t, %d\t, %6.2f\t, %d\t,  %6.4f\t, %s\t, %s%s, %s %s %s %s %s"),
			Index,
			CurrentElement->ChunkDataSize / 1024.0f,
			CurrentElement->Key.ChunkIndex,
			NumTotalChunks,
			NumTimesTouched,
			AveragePlaceInCache,
			CurrentElement->NumConsumers.GetValue(),
			TimeToLoad,
			bWasTrimmed ? TEXT("TRIMMED CHUNK") : *CurrentElement->Key.SoundWaveName.ToString(),
			ShortEnumString(EnumToString(LoadingBehavior)),
			bLoadingBehaviorExternallyOverriden ? TEXT("*") : TEXT(""),
			bWasCacheMiss ? TEXT("(Cache Miss!)") : TEXT(""),
			bIsStaleChunk ? TEXT("(Stale Chunk)") : TEXT(""),
			CurrentElement->IsLoadInProgress() ? TEXT("(Loading In Progress)") : TEXT(""),
			bWasInlinedButUnloaded ? TEXT("(Dumped!)") : bWasLoadedInlined ? TEXT("(Inlined)") : TEXT("(Disk)"),
			bWaveDestroyed ? TEXT("(GCd!)") : TEXT("(Alive)")

		);

		if (!bWasTrimmed)
		{
			OutputString += ElementInfo + TEXT("\n");
		}

		CurrentElement = CurrentElement->LessRecentElement;
		Index++;
	}

	OutputString += TEXT("Cache Miss Log:\n");
	OutputString += FlushCacheMissLog();

	return OutputString;
}


// statics for debug visuals
// Color scheme:
static constexpr float ColorMax = 256.0f;

static const FLinearColor ColorRetainedAndPlaying(40 / ColorMax, 129 / ColorMax, 49 / ColorMax); // Dark Green
static const FLinearColor ColorRetained = FLinearColor::Green; // Light Green

static const FLinearColor ColorPrimedAndPlaying(0, 104 / ColorMax, 174 / ColorMax); // Dark Blue
static const FLinearColor ColorPrimed(65 / ColorMax, 218 / ColorMax, 255 / ColorMax); // Light Blue

static const FLinearColor ColorLODAndPlaying(172 / ColorMax, 128 / ColorMax, 27 / ColorMax); // Dark Yellow
static const FLinearColor ColorLOD(255 / ColorMax, 197 / ColorMax, 1 / ColorMax); // Yellow

static const FLinearColor ColorLoadInProgress = FLinearColor::Black;
static const FLinearColor ColorTrimmed = FLinearColor::Red;
static const FLinearColor ColorCacheMiss = ColorLOD;
static const FLinearColor ColorOther = FLinearColor::Gray;
static const FLinearColor ColorForceInline(255 / ColorMax, 0, 255 / ColorMax); // Magenta
static const FLinearColor ColorExternalFeatures(255 / ColorMax, 100 / ColorMax, 0x00 / ColorMax); // Orange



TPair<int, int> FAudioChunkCache::DebugDisplay(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) const
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

	// Draw our header
	const int32 InitialX = X;
	const int32 InitialY = Y;

	FString NumElementsDetail = *FString::Printf(TEXT("Number of chunks loaded: %d of %d"), ChunksInUse, CachePool.Num());

	const int32 NumCacheOverflows = CacheOverflowCount.GetValue();
	FString CacheOverflowsDetail = *FString::Printf(TEXT("The cache has blown %d times)"), NumCacheOverflows);

	// Offset our number of elements loaded horizontally to the right next to the cache title:
	int32 CacheTitleOffsetX = 0;
	int32 CacheTitleOffsetY = 0;
	UEngine::GetSmallFont()->GetStringHeightAndWidth(TEXT("Cache XX "), CacheTitleOffsetY, CacheTitleOffsetX);

	Canvas->DrawShadowedString(X + CacheTitleOffsetX, Y - 12, *NumElementsDetail, UEngine::GetSmallFont(), FLinearColor::Green);
	Y += 10;

	Canvas->DrawShadowedString(X + CacheTitleOffsetX, Y - 12, *CacheOverflowsDetail, UEngine::GetSmallFont(), NumCacheOverflows != 0 ? FLinearColor::Red : FLinearColor::Green);
	Y += 10;

	// First pass: We run through and get a snap shot of the amount of memory currently in use.
	uint32 NumBytesCounter = 0;

	int32 NumRetainedAndPlaying = 0;
	int32 NumRetained = 0;
	int32 NumPrimedAndPlaying = 0;
	int32 NumPrimed = 0;
	int32 NumRetainedAndPlayingCacheMiss = 0;
	int32 NumRetainedCacheMiss = 0;
	int32 NumPrimedAndPlayingCacheMiss = 0;
	int32 NumPrimedCacheMiss = 0;
	int32 NumLODAndPlaying = 0;
	int32 NumLOD = 0;
	int32 NumTrimmed = 0;
	int32 NumLoadInProgress = 0;
	int32 NumOther = 0;

	for (int i = 0; i < ChunksInUse; ++i)
	{
		const FAudioChunkCache::FCacheElement* CurrentElement = &CachePool[i];

		NumBytesCounter += CurrentElement->ChunkDataSize;

		ESoundWaveLoadingBehavior LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
		bool bWasCacheMiss = false;

		bool bIsPlaying = false;
		const bool bWasTrimmed = CurrentElement->ChunkDataSize == 0;

#if DEBUG_STREAM_CACHE
		bWasCacheMiss = CurrentElement->DebugInfo.bWasCacheMiss;
		LoadingBehavior = CurrentElement->DebugInfo.LoadingBehavior;
		bIsPlaying = CurrentElement->IsBeingPlayed();
#endif
		if (bWasTrimmed)
		{
			++NumTrimmed;
		}
		else if (CurrentElement->IsLoadInProgress())
		{
			++NumLoadInProgress;
		}
		else
		{
			switch (LoadingBehavior)
			{
			case ESoundWaveLoadingBehavior::RetainOnLoad:
				if (bIsPlaying && bWasCacheMiss)
				{
					++NumRetainedAndPlayingCacheMiss;
				}
				else if (bIsPlaying && !bWasCacheMiss)
				{
					++NumRetainedAndPlaying;
				}
				else if (!bIsPlaying && bWasCacheMiss)
				{
					++NumRetainedCacheMiss;
				}
				else if (!bIsPlaying && !bWasCacheMiss)
				{
					++NumRetained;
				}
				break;
			case ESoundWaveLoadingBehavior::PrimeOnLoad:
				if (bIsPlaying && bWasCacheMiss)
				{
					++NumPrimedAndPlayingCacheMiss;
				}
				else if (bIsPlaying && !bWasCacheMiss)
				{
					++NumPrimedAndPlaying;
				}
				else if (!bIsPlaying && bWasCacheMiss)
				{
					++NumPrimedCacheMiss;
				}
				else if (!bIsPlaying && !bWasCacheMiss)
				{
					++NumPrimed;
				}
				break;
			case ESoundWaveLoadingBehavior::LoadOnDemand:
				if (bIsPlaying)
				{
					++NumLODAndPlaying;
				}
				else
				{
					++NumLOD;
				}
				break;
			default:
				++NumOther;
				break;
			}
		}
	}

	uint64 ForceInlineBytes = ForceInlineMemoryCounterBytes->Load();
	uint64 ExternalFeaturesBytes = FeatureMemoryCounterBytes->Load();
	NumBytesCounter += ForceInlineBytes;
	NumBytesCounter += ExternalFeaturesBytes;
	// Convert to megabytes and print the total size:
	const double NumMegabytesInUse = (double)NumBytesCounter / (1024 * 1024);
	const double MaxCacheSizeMB = ((double)MemoryLimitBytes) / (1024 * 1024);

	// calculate ForceInline bytes and percentage
	const double NumMegabytesForceInline = (double)ForceInlineBytes / (1024 * 1024);
	const float PercentageForceInline = NumBytesCounter > 0 ? (double)ForceInlineBytes / NumBytesCounter : 0;

	const double NumMegabytesExternalFeatures = (double)ExternalFeaturesBytes / (1024 * 1024);
	const float PercentageExternalFeatures = NumBytesCounter > 0 ? (double)ExternalFeaturesBytes / NumBytesCounter : 0;

	FString CacheMemoryUsage = *FString::Printf(TEXT("Using: %.4f Megabytes (%" UINT64_FMT " bytes). Max Potential Usage: %.4f Megabytes."), 
		NumMegabytesInUse, GetCurrentMemoryUsageBytes(), MaxCacheSizeMB);

	// We're going to align this horizontally with the number of elements right above it.
	Canvas->DrawShadowedString(X, Y, *CacheMemoryUsage, UEngine::GetMediumFont(), FLinearColor::White);
	Y += 24;

	// gather cache composition as percentages
	float NumChunks = NumRetainedAndPlaying
		+ NumRetained
		+ NumPrimedAndPlaying
		+ NumPrimed
		+ NumRetainedAndPlayingCacheMiss
		+ NumRetainedCacheMiss
		+ NumPrimedAndPlayingCacheMiss
		+ NumPrimedCacheMiss
		+ NumLODAndPlaying
		+ NumLOD
		+ NumTrimmed
		+ NumLoadInProgress
		+ NumOther;

	float PercentageExtra = PercentageForceInline + PercentageExternalFeatures;
	if (FMath::IsNearlyEqual(PercentageExtra, 1.0))
	{
		// if the Percentage is basically 1, then just set the "number of chunks" to a really big number.
		// so everything else just gets zeroed out
		NumChunks = UE_BIG_NUMBER;
	}
	else if (PercentageExtra > 0.0)
	{
		// calculate the NumExtra based on the percentage of memory used.
		int32 NumExtra = NumChunks * (PercentageExtra / (1 - PercentageExtra));

		// derivation:
		// NumChunks + NumExtra = TotalNumChunks
		// NumExtra = TotalNumChunks * PercentageExtra
		// 
		// NumChunks = TotalNumChunks * (1 - PercentageExtra)
		// TotalNumChunks = NumChunks / (1 - PercentageExtra)
		// 
		// - using substitution with the above
		//  NumExtra = NumChunks * PercentageExtra / (1 - PercentageExtra)

		// Add the newly calculated Extra "chunks" to the mix 
		NumChunks += NumExtra;
	}

	if (NumChunks == 0)
	{
		NumChunks = 1.f;
	}

	// Draw the composition bar
	const int32 BarWidth = 0.5f * (Canvas->GetParentCanvasSize().X - 2 * X);
	const int32 BarHeight = 20;
	const int32 BarPad = BarHeight / 7;

	const float PercentageRetainedAndPlaying = (NumRetainedAndPlaying / NumChunks);
	const float PercentageRetained = (NumRetained / NumChunks);
	const float PercentagePrimedAndPlaying = (NumPrimedAndPlaying / NumChunks);
	const float PercentagePrimed = (NumPrimed / NumChunks);
	const float PercentageRetainedAndPlayingCacheMiss = (NumRetainedAndPlayingCacheMiss / NumChunks);
	const float PercentageRetainedCacheMiss = (NumRetainedCacheMiss / NumChunks);
	const float PercentagePrimedAndPlayingCacheMiss = (NumPrimedAndPlayingCacheMiss / NumChunks);
	const float PercentagePrimedCacheMiss = (NumPrimedCacheMiss / NumChunks);
	const float PercentageLODAndPlaying = (NumLODAndPlaying / NumChunks);
	const float PercentageLOD = (NumLOD / NumChunks);
	const float PercentageTrimmed = (NumTrimmed / NumChunks);
	const float PercentageLoadInProgress = (NumLoadInProgress / NumChunks);
	const float PercentageOther = (NumOther / NumChunks);

	const int32 BarWidthRetainedAndPlaying = PercentageRetainedAndPlaying * BarWidth;
	const int32 BarWidthRetained = PercentageRetained * BarWidth;
	const int32 BarWidthPrimedAndPlaying = PercentagePrimedAndPlaying * BarWidth;
	const int32 BarWidthPrimed = PercentagePrimed * BarWidth;
	const int32 BarWidthRetainedAndPlayingCacheMiss = PercentageRetainedAndPlayingCacheMiss * BarWidth;
	const int32 BarWidthRetainedCacheMiss = PercentageRetainedCacheMiss * BarWidth;
	const int32 BarWidthPrimedAndPlayingCacheMiss = PercentagePrimedAndPlayingCacheMiss * BarWidth;
	const int32 BarWidthPrimedCacheMiss = PercentagePrimedCacheMiss * BarWidth;
	const int32 BarWidthLODAndPlaying = PercentageLODAndPlaying * BarWidth;
	const int32 BarWidthLOD = PercentageLOD * BarWidth;
	const int32 BarWidthTrimmed = PercentageTrimmed * BarWidth;
	const int32 BarWidthLoadInProgress = PercentageLoadInProgress * BarWidth;
	const int32 BarWidthOther = PercentageOther * BarWidth;
	const int32 BarWidthForceInline = PercentageForceInline * BarWidth;
	const int32 BarWidthExternalFeatures = PercentageExternalFeatures * BarWidth;


	// Draw color key
	Canvas->DrawShadowedString(X, Y, TEXT("Cache Composition:"), UEngine::GetSmallFont(), FLinearColor::White);
	Y += 15;

	FString TempString = *FString::Printf(TEXT("Retained: %.2f %%"), 100.f * (PercentageRetained + PercentageRetainedAndPlaying));
	Canvas->DrawShadowedString(X, Y, *TempString, UEngine::GetSmallFont(), ColorRetainedAndPlaying);
	Y += 15;

	TempString = *FString::Printf(TEXT("Primed: %.2f %%"), 100.f * (PercentagePrimed + PercentagePrimedAndPlaying));
	Canvas->DrawShadowedString(X, Y, *TempString, UEngine::GetSmallFont(), ColorPrimedAndPlaying);
	Y += 15;

	TempString = *FString::Printf(TEXT("Load On Demand: %.2f %%"), 100.f * (PercentageLOD + PercentageLODAndPlaying));
	Canvas->DrawShadowedString(X, Y, *TempString, UEngine::GetSmallFont(), ColorLODAndPlaying);
	Y += 15;

	TempString = *FString::Printf(TEXT("Trimmed: %.2f %%"), 100.f * PercentageTrimmed);
	Canvas->DrawShadowedString(X, Y, *TempString, UEngine::GetSmallFont(), ColorTrimmed);
	Y += 15;

	TempString = *FString::Printf(TEXT("Load In Progress: %.2f %%"), 100.f * PercentageLoadInProgress);
	Canvas->DrawShadowedString(X, Y, *TempString, UEngine::GetSmallFont(), ColorLoadInProgress);
	Y += 15;

	TempString = *FString::Printf(TEXT("Force Inline: %.2f %% (%.2f MB)"), 100.f * PercentageForceInline, NumMegabytesForceInline);
	Canvas->DrawShadowedString(X, Y, *TempString, UEngine::GetSmallFont(), ColorForceInline);
	Y += 15;

	TempString = *FString::Printf(TEXT("External Features: %.2f %% (%.2f MB)"), 100.f * PercentageExternalFeatures, NumMegabytesExternalFeatures);
	Canvas->DrawShadowedString(X, Y, *TempString, UEngine::GetSmallFont(), ColorExternalFeatures);
	Y += 25;

	TempString = *FString::Printf(TEXT("Other: %.2f %%"), 100.f * PercentageOther);
	Canvas->DrawShadowedString(X, Y, *TempString, UEngine::GetSmallFont(), ColorOther);
	Y += 24;

	int32 CurrHorzOffset = X;
	int32 CurrVertOffset = Y;

	// backdrops
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidth + 2 * BarPad, BarHeight + 4 * BarPad, 0, 0, 0, 0, FLinearColor::Black);
	CurrHorzOffset += BarPad;
	CurrVertOffset += BarPad;

	// cache misses
	// (retained, playing)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthRetainedAndPlayingCacheMiss, BarHeight + 2 * BarPad, 0, 0, 0, 0, ColorCacheMiss);
	CurrHorzOffset += (BarWidthRetainedAndPlayingCacheMiss + BarWidthRetainedAndPlaying);

	// (retained)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthRetainedCacheMiss, BarHeight + 2 * BarPad, 0, 0, 0, 0, ColorCacheMiss);
	CurrHorzOffset += (BarWidthRetainedCacheMiss + BarWidthRetained);

	// (primed, playing)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthPrimedAndPlayingCacheMiss, BarHeight + 2 * BarPad, 0, 0, 0, 0, ColorCacheMiss);
	CurrHorzOffset += (BarWidthPrimedAndPlayingCacheMiss + BarWidthPrimedAndPlaying);

	// (primed)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthPrimedCacheMiss, BarHeight + 2 * BarPad, 0, 0, 0, 0, ColorCacheMiss);
	CurrHorzOffset = X + BarPad;
	CurrVertOffset += BarPad;

	// composition
	// (retained, playing)
	const int32 TotalRetainedAndPlaying = BarWidthRetainedAndPlaying + BarWidthRetainedAndPlayingCacheMiss;
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, TotalRetainedAndPlaying, BarHeight, 0, 0, 0, 0, ColorRetainedAndPlaying);
	CurrHorzOffset += TotalRetainedAndPlaying;

	// (retained)
	const int32 TotalRetained = BarWidthRetained + BarWidthRetainedCacheMiss;
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, TotalRetained, BarHeight, 0, 0, 0, 0, ColorRetained);
	CurrHorzOffset += TotalRetained;

	// (primed, playing)
	const int32 TotalPrimedAndPlaying = BarWidthPrimedAndPlaying + BarWidthPrimedAndPlayingCacheMiss;
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, TotalPrimedAndPlaying, BarHeight, 0, 0, 0, 0, ColorPrimedAndPlaying);
	CurrHorzOffset += TotalPrimedAndPlaying;

	// (primed)
	const int32 TotalPrimed = BarWidthPrimed + BarWidthPrimedCacheMiss;
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, TotalPrimed, BarHeight, 0, 0, 0, 0, ColorPrimed);
	CurrHorzOffset += TotalPrimed;

	// (Load on demand, playing)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthLODAndPlaying, BarHeight, 0, 0, 0, 0, ColorLODAndPlaying);
	CurrHorzOffset += BarWidthLODAndPlaying;

	// (Load on demand)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthLOD, BarHeight, 0, 0, 0, 0, ColorLOD);
	CurrHorzOffset += BarWidthLOD;

	// (Trimmed)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthTrimmed, BarHeight, 0, 0, 0, 0, ColorTrimmed);
	CurrHorzOffset += BarWidthTrimmed;

	// (load in progress)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthLoadInProgress, BarHeight, 0, 0, 0, 0, ColorLoadInProgress);
	CurrHorzOffset += BarWidthLoadInProgress;

	// (other)
	Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthOther, BarHeight, 0, 0, 0, 0, ColorOther);
	CurrHorzOffset += BarWidthOther;
	
	if (BarWidthForceInline > 0 || BarWidthExternalFeatures > 0)
	{
		// (|| divider between cache and chunk memory usage && force inline + External features)
		const int32 DividerWidth = 5;
		Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, DividerWidth, BarHeight, 0, 0, 0, 0, FLinearColor::Black);
		CurrHorzOffset += DividerWidth;

		if (BarWidthForceInline > 0)
		{
			// (Force Inline)
			Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthForceInline - DividerWidth, BarHeight, 0, 0, 0, 0, ColorForceInline);
			CurrHorzOffset += BarWidthForceInline;
		}

		if (BarWidthExternalFeatures > 0)
		{
			// (External Features)
			Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, BarWidthExternalFeatures - DividerWidth, BarHeight, 0, 0, 0, 0, ColorExternalFeatures);
			CurrHorzOffset += BarWidthExternalFeatures;
		}
	}

	Y = (CurrVertOffset + 24);

	// Draw the body of our display depending on the CVAR
	TPair<int, int> Size(X, Y);
	if (DebugViewCVar == 0)
	{
		Size = DebugDisplayLegacy(World, Viewport, Canvas, X, Y + 2 * BarPad, ViewLocation, ViewRotation);
	}
	else if (DebugViewCVar == 1)
	{
		// do nothing else (default)
	}
	else if (DebugViewCVar == 2)
	{
		DebugBirdsEyeDisplay(World, Viewport, Canvas, X, Y, ViewLocation, ViewRotation);
	}
	else if (DebugViewCVar == 3)
	{
		Size = DebugVisualDisplay(World, Viewport, Canvas, X, Y, ViewLocation, ViewRotation);
	}

	return Size;
}

TPair<int, int> FAudioChunkCache::DebugVisualDisplay(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) const
{
	FScopeLock ScopeLock(&CacheMutationCriticalSection.Get());

	// Second Pass: We're going to list the actual chunks in the cache.
	FCacheElement* CurrentElement = MostRecentElement;
	int32 Index = 0;

	const int32 InitialX = X;
	const int32 InitialY = Y;

	float ColorLerpAmount = 0.0f;
	const float ColorLerpStep = 0.04f;

	// More detailed info about individual chunks here:
	const int32 TileSize = 3;
	const int32 TilePadding = 2;
	const int32 MaxWidth = 0.5f * (Canvas->GetParentCanvasSize().X - 2 * X);

	int32 CurrentXOffset = 0;

	// loop over cache chunks
	while (CurrentElement != nullptr)
	{
		// gather chunk info (todo, go through at remove parts that don't alter how a tile is drawn)
		int32 NumTotalChunks = -1;
		int32 NumTimesTouched = -1;
		double TimeToLoad = -1.0;
		float AveragePlaceInCache = -1.0f;
		ESoundWaveLoadingBehavior LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
		bool bLoadingBehaviorExternallyOverriden = false;
		bool bWasCacheMiss = false;
		bool bIsStaleChunk = false;
		bool bIsPlaying = false;

#if DEBUG_STREAM_CACHE
		NumTotalChunks = CurrentElement->DebugInfo.NumTotalChunks;
		NumTimesTouched = CurrentElement->DebugInfo.NumTimesTouched;
		TimeToLoad = CurrentElement->DebugInfo.TimeToLoadMs;
		AveragePlaceInCache = CurrentElement->DebugInfo.AverageLocationInCacheWhenNeeded;
		LoadingBehavior = CurrentElement->DebugInfo.LoadingBehavior;
		bLoadingBehaviorExternallyOverriden = CurrentElement->DebugInfo.bLoadingBehaviorExternallyOverriden;
		bIsPlaying = CurrentElement->IsBeingPlayed();

		// Load on demand is expected to be a cache miss
		bWasCacheMiss = CurrentElement->DebugInfo.bWasCacheMiss && (LoadingBehavior != ESoundWaveLoadingBehavior::LoadOnDemand);
#endif

#if WITH_EDITOR
		// TODO: Worry about whether the sound wave is alive here. In most editor cases this is ok because the soundwave will always be loaded, but this may not be the case in the future.
		bIsStaleChunk = CurrentElement->IsChunkStale();
#endif
		const bool bWasTrimmed = CurrentElement->ChunkDataSize == 0;

		// pick tile color
		FLinearColor TileColor;

		// If there's a load in flight, paint this element yellow.
		if (bWasTrimmed)
		{
			TileColor = ColorTrimmed;
		}
		else if (CurrentElement->IsLoadInProgress())
		{
			TileColor = ColorLoadInProgress;
		}
#if DEBUG_STREAM_CACHE
		else if (LoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad)
		{
			TileColor = bIsPlaying ? ColorRetainedAndPlaying : ColorRetained;
		}
		else if (LoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad)
		{
			TileColor = bIsPlaying ? ColorPrimedAndPlaying : ColorPrimed;
		}
		else if (LoadingBehavior == ESoundWaveLoadingBehavior::LoadOnDemand)
		{
			TileColor = bIsPlaying ? ColorLODAndPlaying : ColorLOD;
		}
#endif
		else
		{
			TileColor = FLinearColor::Gray;
		}


		// draw a tile
		const int32 HalfTilePad = TilePadding / 2;
		const int32 ErrorTileSize = TileSize + TilePadding;

		if (bWasCacheMiss)
		{
			Canvas->DrawTile(X + CurrentXOffset, Y, ErrorTileSize, ErrorTileSize, 0, 0, ErrorTileSize, ErrorTileSize, ColorCacheMiss);
		}
		Canvas->DrawTile(X + CurrentXOffset + HalfTilePad, Y + HalfTilePad, TileSize, TileSize, 0, 0, TileSize, TileSize, TileColor);

		// update "cursor" position
		CurrentXOffset += TileSize + TilePadding;

		// wrap cursor
		if (CurrentXOffset >= MaxWidth)
		{
			CurrentXOffset = 0;
			Y += TileSize + 2 * TilePadding;
		}

		// move to next element
		CurrentElement = CurrentElement->LessRecentElement;
	}

	return TPair<int, int>(X - InitialX, Y - InitialY);
}

TPair<int, int> FAudioChunkCache::DebugBirdsEyeDisplay(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) const
{
	static const int32 DisplayElementSize = 10; // TODO: have this be dynamic based on display size

	const int32 NumChunks = ChunksInUse;
	const FIntPoint CanvasSize = Canvas->GetParentCanvasSize();

	const int32 DisplayWidth = 0.5 * (CanvasSize.X - 2*X);
	const int32 DisplayHeight = DisplayElementSize * 4;

	const int32 NumDisplayElementsHortz = DisplayWidth / DisplayElementSize;
	const int32 NumDisplayElementsVert = 4;
	const int32 NumDisplayElements = NumDisplayElementsHortz * NumDisplayElementsVert;

	const int32 NumChunksPerDispalyElement = FMath::CeilToInt(FMath::Max(1.f, NumChunks / static_cast<float>(NumDisplayElements)));

	int32 CurrHorzOffset = X;
	int32 CurrVertOffset = Y;

	TArray<int32> DebugDisplayCounters;

	FCacheElement* CurrentElement = MostRecentElement;
	while (CurrentElement != nullptr)
	{
		// Reset info
		DebugDisplayCounters.Reset();
		DebugDisplayCounters.SetNumZeroed(static_cast<int32>(EDebugDisplayElementTypes::Count));

		// gather info and draw a single display element
		for (int i = 0; CurrentElement && (i < NumChunksPerDispalyElement); ++i)
		{
			// Gather info
			ESoundWaveLoadingBehavior LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
			bool bWasCacheMiss = false;

			bool bIsPlaying = false;
			const bool bWasTrimmed = CurrentElement->ChunkDataSize == 0;

#if DEBUG_STREAM_CACHE
			bWasCacheMiss = CurrentElement->DebugInfo.bWasCacheMiss;
			LoadingBehavior = CurrentElement->DebugInfo.LoadingBehavior;
			bIsPlaying = CurrentElement->IsBeingPlayed();

#endif
			if (bWasTrimmed)
			{
				++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumTrimmed)];
			}
			else if (CurrentElement->IsLoadInProgress())
			{
				++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumLoadInProgress)];
			}
			else
			{
				switch (LoadingBehavior)
				{
				case ESoundWaveLoadingBehavior::RetainOnLoad:
					if (bIsPlaying && bWasCacheMiss)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumRetainedAndPlayingCacheMiss)];
					}
					else if (bIsPlaying && !bWasCacheMiss)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumRetainedAndPlaying)];
					}
					else if (!bIsPlaying && bWasCacheMiss)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumRetainedCacheMiss)];
					}
					else if (!bIsPlaying && !bWasCacheMiss)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumRetained)];
					}
					break;
				case ESoundWaveLoadingBehavior::PrimeOnLoad:
					if (bIsPlaying && bWasCacheMiss)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumPrimedAndPlayingCacheMiss)];
					}
					else if (bIsPlaying && !bWasCacheMiss)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumPrimedAndPlaying)];
					}
					else if (!bIsPlaying && bWasCacheMiss)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumPrimedCacheMiss)];
					}
					else if (!bIsPlaying && !bWasCacheMiss)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumPrimed)];
					}
					break;
				case ESoundWaveLoadingBehavior::LoadOnDemand:
					if (bIsPlaying)
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumLODAndPlaying)];
					}
					else
					{
						++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumLOD)];
					}
					break;
				default:
					++DebugDisplayCounters[static_cast<int32>(EDebugDisplayElementTypes::NumOther)];
					break;
				}
			}

			CurrentElement = CurrentElement->LessRecentElement;
		}

		// determine the presiding state of the chunks sampled
		int32 MaxValue = -1;
		EDebugDisplayElementTypes PresidingSate = EDebugDisplayElementTypes::NumRetainedAndPlaying;

		// TODO: short-circuit if we know we have a majority
		for (int32 j = 0; j < static_cast<int32>(EDebugDisplayElementTypes::Count); ++j)
		{
			int32& CurrValue = DebugDisplayCounters[j];

			if (CurrValue > MaxValue)
			{
				PresidingSate = static_cast<EDebugDisplayElementTypes>(j);
				MaxValue = CurrValue;
			}
		}

		// Draw display element
		FLinearColor ElementColor = ColorOther;
		switch (PresidingSate)
		{
		case EDebugDisplayElementTypes::NumRetainedAndPlaying:
			ElementColor = ColorRetainedAndPlaying;
			break;
		case EDebugDisplayElementTypes::NumRetained:
			ElementColor = ColorRetained;
			break;
		case EDebugDisplayElementTypes::NumPrimedAndPlaying:
			ElementColor = ColorPrimedAndPlaying;
			break;
		case EDebugDisplayElementTypes::NumPrimed:
			ElementColor = ColorPrimed;
			break;
		case EDebugDisplayElementTypes::NumRetainedAndPlayingCacheMiss:
			ElementColor = ColorRetainedAndPlaying;
			break;
		case EDebugDisplayElementTypes::NumRetainedCacheMiss:
			ElementColor = ColorRetained;
			break;
		case EDebugDisplayElementTypes::NumPrimedAndPlayingCacheMiss:
			ElementColor = ColorPrimedAndPlaying;
			break;
		case EDebugDisplayElementTypes::NumPrimedCacheMiss:
			ElementColor = ColorPrimed;
			break;
		case EDebugDisplayElementTypes::NumLODAndPlaying:
			ElementColor = ColorLODAndPlaying;
			break;
		case EDebugDisplayElementTypes::NumLOD:
			ElementColor = ColorLOD;
			break;
		case EDebugDisplayElementTypes::NumTrimmed:
			ElementColor = ColorTrimmed;
			break;
		case EDebugDisplayElementTypes::NumLoadInProgress:
			ElementColor = ColorLoadInProgress;
			break;
		case EDebugDisplayElementTypes::NumOther:
			ElementColor = ColorOther;
			break;
		default:
			break;
		}

		Canvas->DrawTile(CurrHorzOffset, CurrVertOffset, DisplayElementSize, DisplayElementSize, 0, 0, 0, 0, ElementColor);

		// advance cursor and wrap
		CurrHorzOffset += DisplayElementSize;
		if (CurrHorzOffset >= (X + DisplayWidth))
		{
			CurrHorzOffset = X;
			CurrVertOffset += DisplayElementSize;
		}
	}

	return TPair<int, int>(X, Y);
}
