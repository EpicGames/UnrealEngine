// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AudioThread.cpp: Audio thread implementation.
=============================================================================*/

#include "AudioThread.h"
#include "Audio.h"
#include "Async/Async.h"
#include "Containers/SpscQueue.h"
#include "Containers/Ticker.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Tasks/Pipe.h"

//
// Globals
//

extern CORE_API UE::Tasks::FPipe GAudioPipe;
extern CORE_API std::atomic<bool> GIsAudioThreadRunning;
extern CORE_API std::atomic<bool> GIsAudioThreadSuspended;

static int32 GCVarSuspendAudioThread = 0;
FAutoConsoleVariableRef CVarSuspendAudioThread(TEXT("AudioThread.SuspendAudioThread"), GCVarSuspendAudioThread, TEXT("0=Resume, 1=Suspend"), ECVF_Cheat);

static int32 GCVarAboveNormalAudioThreadPri = 0;
FAutoConsoleVariableRef CVarAboveNormalAudioThreadPri(TEXT("AudioThread.AboveNormalPriority"), GCVarAboveNormalAudioThreadPri, TEXT("0=Normal, 1=AboveNormal"), ECVF_Default);

static int32 GCVarEnableAudioCommandLogging = 0;
FAutoConsoleVariableRef CVarEnableAudioCommandLogging(TEXT("AudioThread.EnableAudioCommandLogging"), GCVarEnableAudioCommandLogging, TEXT("0=Disbaled, 1=Enabled"), ECVF_Default);

static int32 GCVarEnableBatchProcessing = 1;
FAutoConsoleVariableRef CVarEnableBatchProcessing(
	TEXT("AudioThread.EnableBatchProcessing"),
	GCVarEnableBatchProcessing,
	TEXT("Enables batch processing audio thread commands.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default); 

static int32 GBatchAudioAsyncBatchSize = 128;
static FAutoConsoleVariableRef CVarBatchAudioAsyncBatchSize(
	TEXT("AudioThread.BatchAsyncBatchSize"),
	GBatchAudioAsyncBatchSize,
	TEXT("When AudioThread.EnableBatchProcessing = 1, controls the number of audio commands grouped together for threading.")
);

static int32 GAudioCommandFenceWaitTimeMs = 35;
FAutoConsoleVariableRef  CVarAudioCommandFenceWaitTimeMs(
	TEXT("AudioCommand.FenceWaitTimeMs"),
	GAudioCommandFenceWaitTimeMs, 
	TEXT("Sets number of ms for fence wait"), 
	ECVF_Default);

namespace AudioCommandPrivate
{
	bool bBatchGameThreadAudioCommands = true;
	
	/*
	 * An audio command to be executed on the game thread.
	 */
	struct FAudioCommand
	{
		FAudioCommand() = default;

		FAudioCommand(TUniqueFunction<void()> InFunction, const TStatId InStatId)
			: Function(MoveTemp(InFunction))
			, StatId(InStatId)
		{
		}

		FAudioCommand(FAudioCommand&& Other) = default;
		FAudioCommand& operator=(FAudioCommand&& Other) = default;

		static void Execute(const TUniqueFunction<void()>& InFunction, TStatId InStatId)
		{
			check(IsInGameThread());

			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Audio);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_AudioCommandQueue_RunCommandOnGameThread);
			FScopeCycleCounter ScopeCycleCounter(InStatId);

			InFunction();
		}

		TUniqueFunction<void()> Function;
		TStatId StatId;
	};

	const FLazyName DiagnosticName("FGameThreadAudioCommandQueue");

	/*
	 * Used to run commands from the audio thread, on the game thread.
	 */
	class FGameThreadAudioCommandQueue : public FTickFunction
	{
	public:
		FGameThreadAudioCommandQueue()
			: FTickFunction()
		{
			// Setup tick properties 
			bCanEverTick = true;
			bTickEvenWhenPaused = false;
			bStartWithTickEnabled = true;
			TickGroup = TG_PrePhysics;
			bAllowTickOnDedicatedServer = false;
			bRunOnAnyThread = false;
			bIsTicking = false;
		}

		virtual ~FGameThreadAudioCommandQueue() override = default;

		void StartTick()
		{
			if (!bIsTicking && bBatchGameThreadAudioCommands)
			{
				if (GEngine)
				{
					RegisterTickFunctionToWorld(GEngine->GetCurrentPlayWorld());
				}
			
				FWorldDelegates::OnWorldInitializedActors.AddRaw(this, &FGameThreadAudioCommandQueue::OnWorldActorsInitialized);
				bIsTicking = true;
			}
		}

		void EndTick()
		{
			if (bIsTicking)
			{
				FWorldDelegates::OnWorldInitializedActors.RemoveAll(this);
				UnRegisterTickFunction();
				bIsTicking = false;
			}
		}

		bool CanTick() const
		{
			return bBatchGameThreadAudioCommands && bIsTicking;
		}

		void RunCommandOnGameThread(TUniqueFunction<void()> InFunction, const TStatId InStatId)
		{
			if (IsAudioThreadRunning())
			{
				check(IsInAudioThread());
				Commands.Enqueue(FAudioCommand(MoveTemp(InFunction), InStatId));
			}
			else
			{
				FAudioCommand::Execute(InFunction, InStatId);
			}
		}

		void FlushCommands()
		{
			TOptional<FAudioCommand> AudioCommand = Commands.Dequeue();
			while (AudioCommand.IsSet())
			{
				if (IsInAudioThread())
				{
					ExecuteOnGameThread(
						TEXT("FAudioThread::RunCommandOnGameThread"),
						[Function = MoveTemp(AudioCommand.GetValue().Function), StatId = AudioCommand.GetValue().StatId]()
						{
							FAudioCommand::Execute(Function, StatId);
						});
				}
				else
				{
					FAudioCommand::Execute(AudioCommand.GetValue().Function, AudioCommand.GetValue().StatId);
				}
				
				AudioCommand = Commands.Dequeue();
			}
		}

	private:
		// Begin FTickFunction overrides
		virtual void ExecuteTick(float InDeltaTime, ELevelTick InTickType, ENamedThreads::Type InCurrentThread, const FGraphEventRef& InMyCompletionGraphEvent) override
		{
			TOptional<FAudioCommand> AudioCommand = Commands.Dequeue();
			while (AudioCommand.IsSet())
			{
				FAudioCommand::Execute(AudioCommand.GetValue().Function, AudioCommand.GetValue().StatId);
				AudioCommand = Commands.Dequeue();
			}
		}

		virtual FString DiagnosticMessage() override
		{
			return DiagnosticName.ToString();
		}

		virtual FName DiagnosticContext(bool bDetailed) override
		{
			return DiagnosticName;
		}
		// End FTickFunction overrides
		
		void OnWorldActorsInitialized(const UWorld::FActorsInitializedParams& InParams)
		{
			// When a new world is up and running unregister the tick function with the old world
			// and register it with the new world
			UnRegisterTickFunction();
			RegisterTickFunctionToWorld(InParams.World);
		}
		
		void RegisterTickFunctionToWorld(const UWorld* const InWorld)
		{
			if (InWorld)
			{
				RegisterTickFunction(InWorld->PersistentLevel);
			}
		}
		
		TSpscQueue<FAudioCommand> Commands;
		bool bIsTicking;
	};
	
	FGameThreadAudioCommandQueue CommandQueue;

	void BatchGameThreadAudioCommandsChangedCallback(IConsoleVariable* ConsoleVariable)
	{
		if (GIsAudioThreadRunning)
		{
			const bool bBatchCommands = ConsoleVariable->GetBool();
			if (bBatchCommands)
			{
				CommandQueue.StartTick();
			}
			else
			{
				CommandQueue.EndTick();
				CommandQueue.FlushCommands();
			}
		}
	}

	FAutoConsoleVariableRef CVarBatchGameThreadAudioCommands(
		TEXT("AudioThread.BatchCommands"),
		bBatchGameThreadAudioCommands,
		TEXT("Batch audio commands that are created from the audio thread and executed on the game thread so that they are executed in a single task."),
		FConsoleVariableDelegate::CreateStatic(&BatchGameThreadAudioCommandsChangedCallback),
		ECVF_Default
		);
}

static bool GAudioThreadUseSafeRunCommandOnGameThread = true;
FAutoConsoleVariableRef CVarAudioThreadUseSafeRunCommandOnGameThread(
	TEXT("AudioThread.UseSafeRunCommandOnGameThread"),
	GAudioThreadUseSafeRunCommandOnGameThread,
	TEXT("When active, limit commands sent to the game thread to run at a safe place inside a tick"),
	ECVF_Default);

struct FAudioThreadInteractor
{
	static void UseAudioThreadCVarSinkFunction()
	{
		static bool bLastSuspendAudioThread = false;
		const bool bSuspendAudioThread = GCVarSuspendAudioThread != 0;

		if (bLastSuspendAudioThread != bSuspendAudioThread)
		{
			bLastSuspendAudioThread = bSuspendAudioThread;
			if (bSuspendAudioThread && IsAudioThreadRunning())
			{
				FAudioThread::SuspendAudioThread();
			}
			else if (GIsAudioThreadSuspended)
			{
				FAudioThread::ResumeAudioThread();
			}
			else if (GIsEditor)
			{
				UE_LOGF(LogAudio, Warning, "Audio threading is disabled in the editor.");
			}
			else if (!FAudioThread::IsUsingThreadedAudio())
			{
				UE_LOGF(LogAudio, Warning, "Cannot manipulate audio thread when disabled by platform or ini.");
			}
		}
	}
};

UE::Tasks::ETaskPriority GAudioTaskPriority = UE::Tasks::ETaskPriority::Normal;

static void SetAudioTaskPriority(const TArray<FString>& Args)
{
	UE_LOGF(LogConsoleResponse, Display, "AudioTaskPriority was %ls.", LowLevelTasks::ToString(GAudioTaskPriority));

	if (Args.Num() > 1)
	{
		UE_LOGF(LogConsoleResponse, Display, "WARNING: This command requires a single argument while %d were provided, all extra arguments will be ignored.", Args.Num());
	}
	else if (Args.IsEmpty())
	{
		UE_LOGF(LogConsoleResponse, Display, "ERROR: Please provide a new priority value.");
		return;
	}

	if (!LowLevelTasks::ToTaskPriority(*Args[0], GAudioTaskPriority))
	{
		UE_LOGF(LogConsoleResponse, Display, "ERROR: Invalid priority: %ls.", *Args[0]);
	}

	UE_LOGF(LogConsoleResponse, Display, "Audio Task Priority was set to %ls.", LowLevelTasks::ToString(GAudioTaskPriority));
}

static FAutoConsoleCommand AudioThreadPriorityConsoleCommand(
	TEXT("AudioThread.TaskPriority"),
	TEXT("Takes a single parameter of value `High`, `Normal`, `BackgroundHigh`, `BackgroundNormal` or `BackgroundLow`."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&SetAudioTaskPriority)
);

static FAutoConsoleVariableSink CVarUseAudioThreadSink(FConsoleCommandDelegate::CreateStatic(&FAudioThreadInteractor::UseAudioThreadCVarSinkFunction));

bool FAudioThread::bUseThreadedAudio = false;

FCriticalSection FAudioThread::CurrentAudioThreadStatIdCS;
TStatId FAudioThread::CurrentAudioThreadStatId;
TStatId FAudioThread::LongestAudioThreadStatId;
double FAudioThread::LongestAudioThreadTimeMsec = 0.0;

TUniquePtr<UE::Tasks::FTaskEvent> FAudioThread::ResumeEvent;
int32 FAudioThread::SuspendCount{ 0 };

void FAudioThread::SuspendAudioThread()
{
	check(IsInGameThread()); // thread-safe version would be much more complicated

	if (!GIsAudioThreadRunning)
	{
		return; // nothing to suspend
	}

	if (++SuspendCount != 1)
	{
		return; // recursive scope
	}

	check(!GIsAudioThreadSuspended.load(std::memory_order_relaxed));

	FEventRef SuspendEvent;

	FAudioThread::RunCommandOnAudioThread(
		[&SuspendEvent]
		{
			GIsAudioThreadSuspended.store(true, std::memory_order_release);
			UE::Tasks::AddNested(*ResumeEvent);
			SuspendEvent->Trigger();
		}
	);

	// release batch processing so the task above will be executed
	FAudioThread::ProcessAllCommands();

	// wait for the command above to block audio processing
	SuspendEvent->Wait();
}

void FAudioThread::ResumeAudioThread()
{
	check(IsInGameThread());

	if (!GIsAudioThreadRunning)
	{
		return; // nothing to resume
	}

	if (--SuspendCount != 0)
	{
		return; // recursive scope
	}

	check(GIsAudioThreadSuspended.load(std::memory_order_relaxed));
	GIsAudioThreadSuspended.store(false, std::memory_order_release);

	check(!ResumeEvent->IsCompleted());
	ResumeEvent->Trigger();
	ResumeEvent = MakeUnique<UE::Tasks::FTaskEvent>(UE_SOURCE_LOCATION);
}

void FAudioThread::SetUseThreadedAudio(const bool bInUseThreadedAudio)
{
	if (IsAudioThreadRunning() && !bInUseThreadedAudio)
	{
		UE_LOGF(LogAudio, Error, "You cannot disable using threaded audio once the thread has already begun running.");
	}
	else
	{
		bUseThreadedAudio = bInUseThreadedAudio;
	}
}

bool FAudioThread::IsUsingThreadedAudio()
{
	return bUseThreadedAudio;
}

// batching audio commands allows to avoid the overhead of launching a task per command when resources are limited.
// We assume that resources are limited if the previous batch is not completed yet (potentially waiting for execution due to the CPU being busy
// with something else). Otherwise we don't wait until we collect a full batch.
struct FAudioAsyncBatcher
{
	using FWork = TUniqueFunction<void()>;
	TArray<FWork> WorkItems;

	UE::Tasks::FTask LastBatch;

	void Add(FWork&& Work)
	{
		check(IsInGameThread());

#if !WITH_EDITOR
		if (GCVarEnableBatchProcessing)
		{
			if (WorkItems.Num() >= GBatchAudioAsyncBatchSize) // collected enough work
			{
				Flush();
			}
			WorkItems.Add(Forward<FWork>(Work));
		}
#else
		LastBatch = GAudioPipe.Launch(UE_SOURCE_LOCATION, MoveTemp(Work));
#endif
	}

	void Flush()
	{
		check(IsInGameThread());

		if (WorkItems.IsEmpty())
		{
			return;
		}

		LastBatch = GAudioPipe.Launch(TEXT("AudioBatch"),
			[WorkItems = MoveTemp(WorkItems)]() mutable
			{
				LLM_SCOPE(ELLMTag::AudioMisc);

				for (FWork& Work : WorkItems)
				{
					Work();
				}
			},
			GAudioTaskPriority
		);
		WorkItems.Reset();
	}
};

static FAudioAsyncBatcher GAudioAsyncBatcher;

TUniqueFunction<void()> FAudioThread::GetCommandWrapper(TUniqueFunction<void()> InFunction, const TStatId InStatId)
{
	if (GCVarEnableAudioCommandLogging == 1)
	{
		return [Function = MoveTemp(InFunction), InStatId]()
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Audio);

			FScopeCycleCounter ScopeCycleCounter(InStatId);
			FAudioThread::SetCurrentAudioThreadStatId(InStatId);

			// Time the execution of the function
			const double StartTime = FPlatformTime::Seconds();

			// Execute the function
			Function();

			// Track the longest one
			const double DeltaTime = (FPlatformTime::Seconds() - StartTime) * 1000.0f;
			if (DeltaTime > GetCurrentLongestTime())
			{
				SetLongestTimeAndId(InStatId, DeltaTime);
			}
		};
	}
	else
	{
		return [Function = MoveTemp(InFunction), InStatId]()
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Audio);

			FScopeCycleCounter ScopeCycleCounter(InStatId);
			Function();
		};
	}
}

void FAudioThread::RunCommandOnAudioThread(TUniqueFunction<void()> InFunction, const TStatId InStatId)
{
	TUniqueFunction<void()> CommandWrapper{ GetCommandWrapper(MoveTemp(InFunction), InStatId) };
	if (IsInAudioThread())
	{
		// it's audio-thread-safe so execute the command in-place
		CommandWrapper();
	}
	else if (IsInGameThread())
	{
		// batch commands to minimise game thread overhead
		GAudioAsyncBatcher.Add(MoveTemp(CommandWrapper));
	}
	// we are on an unknown thread
	else if (IsUsingThreadedAudio())
	{
		GAudioPipe.Launch(TEXT("AudioCommand"), MoveTemp(CommandWrapper), GAudioTaskPriority);
	}
	else
	{
		// the command must be executed on the game thread
		AsyncTask(ENamedThreads::GameThread, MoveTemp(CommandWrapper));
	}
}

void FAudioThread::SetCurrentAudioThreadStatId(TStatId InStatId)
{
	FScopeLock Lock(&CurrentAudioThreadStatIdCS);
	CurrentAudioThreadStatId = InStatId;
}

FString FAudioThread::GetCurrentAudioThreadStatId()
{
	FScopeLock Lock(&CurrentAudioThreadStatIdCS);
#if STATS
	return FString(CurrentAudioThreadStatId.GetStatDescriptionANSI());
#else
	return FString("NoStats");
#endif
}

void FAudioThread::ResetAudioThreadTimers()
{
	FScopeLock Lock(&CurrentAudioThreadStatIdCS);
	LongestAudioThreadStatId = TStatId();
	LongestAudioThreadTimeMsec = 0.0;
}

void FAudioThread::SetLongestTimeAndId(TStatId NewLongestId, double LongestTimeMsec)
{
	FScopeLock Lock(&CurrentAudioThreadStatIdCS);
	LongestAudioThreadTimeMsec = LongestTimeMsec;
	LongestAudioThreadStatId = NewLongestId;
}

void FAudioThread::GetLongestTaskInfo(FString& OutLongestTask, double& OutLongestTaskTimeMs)
{
	FScopeLock Lock(&CurrentAudioThreadStatIdCS);
#if STATS
	OutLongestTask = FString(LongestAudioThreadStatId.GetStatDescriptionANSI());
#else
	OutLongestTask = FString("NoStats");
#endif
	OutLongestTaskTimeMs = LongestAudioThreadTimeMsec;
}

void FAudioThread::ProcessAllCommands()
{
	if (IsAudioThreadRunning())
	{
		GAudioAsyncBatcher.Flush();
	}
	else
	{
		check(GAudioAsyncBatcher.WorkItems.IsEmpty());
	}
}

void FAudioThread::RunCommandOnGameThread(TUniqueFunction<void()> InFunction, const TStatId InStatId)
{
	if (AudioCommandPrivate::CommandQueue.CanTick())
	{
		AudioCommandPrivate::CommandQueue.RunCommandOnGameThread(MoveTemp(InFunction), InStatId);
	}
	else
	{
	if (IsAudioThreadRunning())
	{
		check(IsInAudioThread());
		if (GAudioThreadUseSafeRunCommandOnGameThread)
		{
			ExecuteOnGameThread(
				TEXT("FAudioThread::RunCommandOnGameThread"),
				[Function = MoveTemp(InFunction), InStatId]()
				{
					CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Audio);
					QUICK_SCOPE_CYCLE_COUNTER(STAT_AudioThread_RunCommandOnGameThread);
					FScopeCycleCounter ScopeCycleCounter(InStatId);
					Function();
				}
			);
		}
		else
		{
			// This is the legacy behavior that will run game thread tasks anywhere on the game thread.
			// This could be inside the GC, postload, etc... which might lead to problematic behavior.
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[Function = MoveTemp(InFunction), InStatId]()
				{
					CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Audio);
					QUICK_SCOPE_CYCLE_COUNTER(STAT_AudioThread_RunCommandOnGameThread);
					FScopeCycleCounter ScopeCycleCounter(InStatId);
					Function();
				},
				TStatId(),
				nullptr,
				ENamedThreads::GameThread);
		}
	}
	else
	{
		check(IsInGameThread());
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Audio);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_AudioThread_RunCommandOnGameThread);
		FScopeCycleCounter ScopeCycleCounter(InStatId);
		InFunction();
	}
}
}

FDelegateHandle FAudioThread::PreGC;
FDelegateHandle FAudioThread::PostGC;
FDelegateHandle FAudioThread::PreGCDestroy;
FDelegateHandle FAudioThread::PostGCDestroy;

void FAudioThread::StartAudioThread()
{
	check(IsInGameThread());

	check(!GIsAudioThreadRunning.load(std::memory_order_relaxed));

	if (!bUseThreadedAudio)
	{
		return;
	}

	PreGC = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddStatic(&FAudioThread::SuspendAudioThread);
	PostGC = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&FAudioThread::ResumeAudioThread);

	PreGCDestroy = FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy.AddStatic(&FAudioThread::SuspendAudioThread);
	PostGCDestroy = FCoreUObjectDelegates::PostGarbageCollectConditionalBeginDestroy.AddStatic(&FAudioThread::ResumeAudioThread);

	check(!ResumeEvent.IsValid());
	ResumeEvent = MakeUnique<UE::Tasks::FTaskEvent>(UE_SOURCE_LOCATION);

	AudioCommandPrivate::CommandQueue.StartTick();

	GIsAudioThreadRunning.store(true, std::memory_order_release);
}

void FAudioThread::StopAudioThread()
{
	if (!IsAudioThreadRunning())
	{
		return;
	}

	GAudioAsyncBatcher.Flush();
	GAudioAsyncBatcher.LastBatch.Wait();
	GAudioAsyncBatcher.LastBatch = UE::Tasks::FTask{}; // release the task as it can hold some references

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(PreGC);
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGC);
	FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy.Remove(PreGCDestroy);
	FCoreUObjectDelegates::PostGarbageCollectConditionalBeginDestroy.Remove(PostGCDestroy);

	GIsAudioThreadRunning.store(false, std::memory_order_release);

	check(ResumeEvent.IsValid());
	ResumeEvent->Trigger(); // every FTaskEvent must be triggered before destruction to pass the check for completion
	ResumeEvent.Reset();
	
	AudioCommandPrivate::CommandQueue.EndTick();
	AudioCommandPrivate::CommandQueue.FlushCommands();
}

FAudioCommandFence::~FAudioCommandFence()
{
	check(IsInGameThread());
	CSV_SCOPED_SET_WAIT_STAT(Audio);
	Fence.Wait();
}

void FAudioCommandFence::BeginFence()
{
	check(IsInGameThread());

	if (!IsAudioThreadRunning())
	{
		return;
	}

	{
		CSV_SCOPED_SET_WAIT_STAT(Audio);
		Fence.Wait();
	}
	
	FAudioThread::ProcessAllCommands();
	Fence = GAudioAsyncBatcher.LastBatch;
}

bool FAudioCommandFence::IsFenceComplete() const
{
	check(IsInGameThread());
	return Fence.IsCompleted();
}

/**
 * Waits for pending fence commands to retire.
 */
void FAudioCommandFence::Wait(bool bProcessGameThreadTasks) const
{
	check(IsInGameThread());

	{
		CSV_SCOPED_SET_WAIT_STAT(Audio);
		Fence.Wait();
	}

	Fence = UE::Tasks::FTask{}; // release the task as it can hold some references
}
