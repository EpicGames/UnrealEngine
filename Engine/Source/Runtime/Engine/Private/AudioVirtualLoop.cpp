// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioVirtualLoop.h"

#include "Audio/AudioDebug.h"
#include "AudioDevice.h"
#include "AudioMixerTrace.h"


static int32 bVirtualLoopsEnabledCVar = 1;
FAutoConsoleVariableRef CVarVirtualLoopsEnabled(
	TEXT("au.VirtualLoops.Enabled"),
	bVirtualLoopsEnabledCVar,
	TEXT("Enables or disables whether virtualizing is supported for audio loops.\n"),
	ECVF_Default);

static float VirtualLoopsPerfDistanceCVar = 15000.0f;
FAutoConsoleVariableRef CVarVirtualLoopsPerfDistance(
	TEXT("au.VirtualLoops.PerfDistance"),
	VirtualLoopsPerfDistanceCVar,
	TEXT("Sets virtual loop distance to scale update rate between min and max beyond max audible distance of sound.\n"),
	ECVF_Default);

static float VirtualLoopsForceUpdateListenerMoveDistanceCVar = 2500.0f;
FAutoConsoleVariableRef CVarVirtualLoopsForceUpdateListenerMoveDistance(
	TEXT("au.VirtualLoops.ForceUpdateListenerMoveDistance"),
	VirtualLoopsForceUpdateListenerMoveDistanceCVar,
	TEXT("Sets distance threshold required to force an update on virtualized sounds to check for if listener moves in a single frame over the given distance.\n"),
	ECVF_Default);

static float VirtualLoopsUpdateRateMinCVar = 0.1f;
FAutoConsoleVariableRef CVarVirtualLoopsUpdateRateMin(
	TEXT("au.VirtualLoops.UpdateRate.Min"),
	VirtualLoopsUpdateRateMinCVar,
	TEXT("Sets minimum rate to check if sound becomes audible again at sound's max audible distance.\n"),
	ECVF_Default);

static float VirtualLoopsUpdateRateMaxCVar = 3.0f;
FAutoConsoleVariableRef CVarVirtualLoopsUpdateRateMax(
	TEXT("au.VirtualLoops.UpdateRate.Max"),
	VirtualLoopsUpdateRateMaxCVar,
	TEXT("Sets maximum rate to check if sound becomes audible again (at beyond sound's max audible distance + perf scaling distance).\n"),
	ECVF_Default);


#if UE_AUDIO_PROFILERTRACE_ENABLED
UE_TRACE_EVENT_BEGIN(Audio, VirtualLoopVirtualize)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint64, ComponentId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, VirtualLoopIsVirtualizedPing)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint64, ComponentId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, VirtualLoopUpdate)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(float, TimeVirtualized)
	UE_TRACE_EVENT_FIELD(float, PlaybackTime)
	UE_TRACE_EVENT_FIELD(float, UpdateInterval)
	UE_TRACE_EVENT_FIELD(double, LocationX)
	UE_TRACE_EVENT_FIELD(double, LocationY)
	UE_TRACE_EVENT_FIELD(double, LocationZ)
	UE_TRACE_EVENT_FIELD(double, RotatorPitch)
	UE_TRACE_EVENT_FIELD(double, RotatorYaw)
	UE_TRACE_EVENT_FIELD(double, RotatorRoll)

UE_TRACE_EVENT_END()
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

namespace AudioVirtualLoopPrivate
{
	static const FLazyName VolumeModulationParameterName("Volume");
	static const FLazyName PitchModulationParameterName("Pitch");
}

FAudioVirtualLoop::FAudioVirtualLoop()
	: TimeSinceLastUpdate(0.0f)
	, TimeVirtualized(0.0f)
	, UpdateInterval(0.0f)
	, ActiveSound(nullptr)
{
}

bool FAudioVirtualLoop::Virtualize(const FActiveSound& InActiveSound, bool bDoRangeCheck, FAudioVirtualLoop& OutVirtualLoop)
{
	FAudioDevice* AudioDevice = InActiveSound.AudioDevice;
	check(AudioDevice);

	return Virtualize(InActiveSound, *AudioDevice, bDoRangeCheck, OutVirtualLoop);
}

bool FAudioVirtualLoop::Virtualize(const FActiveSound& InActiveSound, FAudioDevice& InAudioDevice, bool bDoRangeCheck, FAudioVirtualLoop& OutVirtualLoop)
{
	USoundBase* Sound = InActiveSound.GetSound();
	check(Sound);

	if (InActiveSound.GetVirtualizationMode() == EVirtualizationMode::Disabled)
	{
		return false;
	}

	if (!bVirtualLoopsEnabledCVar || InActiveSound.bIsPreviewSound || !InActiveSound.IsLooping())
	{
		return false;
	}

	if (InActiveSound.FadeOut != FActiveSound::EFadeOut::None || InActiveSound.bIsStopping)
	{
		return false;
	}

	if (InAudioDevice.CanHaveMultipleActiveSounds(InActiveSound.GetAudioComponentID()))
	{
		return false;
	}

	if (bDoRangeCheck && IsInAudibleRange(InActiveSound, &InAudioDevice))
	{
		return false;
	}

	FActiveSound* ActiveSound = FActiveSound::CreateVirtualCopy(InActiveSound, InAudioDevice);
	OutVirtualLoop.ActiveSound = ActiveSound;
	OutVirtualLoop.CalculateUpdateInterval();

	if (InActiveSound.GetVirtualizationMode() == EVirtualizationMode::SeekRestart)
	{
		for (const TPair<UPTRINT, FWaveInstance*>& Pair : InActiveSound.GetWaveInstances())
		{
			if (const FWaveInstance* WaveInstance = Pair.Value)
			{
				FWaveInstanceData NewWaveInstanceData;
				NewWaveInstanceData.EffectivePitch = WaveInstance->Pitch;

				// Each Wave Instance can have its own assigned sound class, which can modify the pitch while the sound is virtualized.
				// This has an impact on the effective playback time, so we have to track it for Seek Restart.
				if (WaveInstance->SoundClass)
				{
					NewWaveInstanceData.SoundClass = WaveInstance->SoundClass;
				}
			
				// Same with Pitch modulation
				const FSoundModulationDestinationSettings& PitchModulationSettings = WaveInstance->GetEffectiveModulationSettings(EModulationDestination::Pitch);
				if (!PitchModulationSettings.Modulators.IsEmpty())
				{
					NewWaveInstanceData.PitchModulationDestination.Init(InAudioDevice.DeviceID, AudioVirtualLoopPrivate::PitchModulationParameterName);
					NewWaveInstanceData.PitchModulationDestination.SetModulators(PitchModulationSettings.Modulators);
				}
				
				OutVirtualLoop.WaveInstanceData.Add(Pair.Key, NewWaveInstanceData);
				OutVirtualLoop.EffectivePlaybackTimes.Add(Pair.Key, WaveInstance->PlaybackPercent * WaveInstance->WaveData->Duration);
			}
		}
	}

	// Once virtualized, Modulation stops getting updated at the Mixer Source level. Because of this, we need to grab
	// Modulation data now so that Volume modulation can be used to determine if a sound is loud enough to realize.
	// It's possible that the ActiveSound contains multiple WaveInstances.
	// Because of this, we need to store one Modulation Destination for each WaveInstance.
	TArray<TSet<TObjectPtr<USoundModulatorBase>>> ModulatorSets = ActiveSound->GetEffectiveVolumeModulators();
	for (int32 DestinationIndex = 0; DestinationIndex < ModulatorSets.Num(); ++DestinationIndex)
	{
		OutVirtualLoop.VolumeConcurrencyDestinations.Add(Audio::FModulationDestination());
		OutVirtualLoop.VolumeConcurrencyDestinations[DestinationIndex].Init(InAudioDevice.DeviceID, AudioVirtualLoopPrivate::VolumeModulationParameterName, false /* bInIsBuffered */, true /* bInValueNormalized */);
		OutVirtualLoop.VolumeConcurrencyDestinations[DestinationIndex].SetModulators(ModulatorSets[DestinationIndex]);
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
	if (bChannelEnabled)
	{
		if (const FAudioDevice* AudioDevice = ActiveSound->AudioDevice)
		{
			UE_TRACE_LOG(Audio, VirtualLoopVirtualize, AudioChannel)
				<< VirtualLoopVirtualize.DeviceId(static_cast<uint32>(AudioDevice->DeviceID))
				<< VirtualLoopVirtualize.Timestamp(FPlatformTime::Cycles64())
				<< VirtualLoopVirtualize.PlayOrder(ActiveSound->GetPlayOrder())
				<< VirtualLoopVirtualize.ComponentId(ActiveSound->GetAudioComponentID())
				<< VirtualLoopVirtualize.Name(Sound ? *Sound->GetPathName() : TEXT("N/A"));
		}
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	return true;
}

void FAudioVirtualLoop::CalculateUpdateInterval()
{
	check(ActiveSound);
	FAudioDevice* AudioDevice = ActiveSound->AudioDevice;
	check(AudioDevice);

	const float DistanceToListener = AudioDevice->GetDistanceToNearestListener(ActiveSound->Transform.GetLocation());
	const float DistanceRatio = (DistanceToListener - ActiveSound->MaxDistance) / FMath::Max(VirtualLoopsPerfDistanceCVar, 1.0f);
	const float DistanceRatioClamped = FMath::Clamp(DistanceRatio, 0.0f, 1.0f);
	UpdateInterval = FMath::Lerp(VirtualLoopsUpdateRateMinCVar, VirtualLoopsUpdateRateMaxCVar, DistanceRatioClamped);
}

float FAudioVirtualLoop::GetTimeVirtualized() const
{
	return TimeVirtualized;
}

const TSortedMap<UPTRINT, float>& FAudioVirtualLoop::GetEffectivePlaybackTimes()
{
	return EffectivePlaybackTimes;
}

float FAudioVirtualLoop::GetUpdateInterval() const
{
	return UpdateInterval;
}

FActiveSound& FAudioVirtualLoop::GetActiveSound()
{
	check(ActiveSound);
	return *ActiveSound;
}

const FActiveSound& FAudioVirtualLoop::GetActiveSound() const
{
	check(ActiveSound);
	return *ActiveSound;
}

bool FAudioVirtualLoop::IsEnabled()
{
	return bVirtualLoopsEnabledCVar != 0;
}

bool FAudioVirtualLoop::IsInAudibleRange(const FActiveSound& InActiveSound, const FAudioDevice* InAudioDevice)
{
	if (!InActiveSound.bAllowSpatialization)
	{
		return true;
	}

	const FAudioDevice* AudioDevice = InAudioDevice;
	if (!AudioDevice)
	{
		AudioDevice = InActiveSound.AudioDevice;
	}
	check(AudioDevice);

	if (InActiveSound.IsPlayWhenSilent())
	{
		return true;
	}

	float DistanceScale = 1.0f;
	if (InActiveSound.bHasAttenuationSettings)
	{
		// If we are not using distance-based attenuation, this sound will be audible regardless of distance.
		if (!InActiveSound.AttenuationSettings.bAttenuate)
		{
			return true;
		}

		DistanceScale = InActiveSound.FocusData.DistanceScale;
	}

	DistanceScale = FMath::Max(DistanceScale, UE_KINDA_SMALL_NUMBER);
	const FVector Location = InActiveSound.Transform.GetLocation();
	return AudioDevice->LocationIsAudible(Location, InActiveSound.MaxDistance / DistanceScale);
}

void FAudioVirtualLoop::UpdateFocusData(float DeltaTime)
{
	check(ActiveSound);

	if (!ActiveSound->bHasAttenuationSettings)
	{
		return;
	}

	// If we are not using distance-based attenuation, this sound will be audible regardless of distance.
	if (!ActiveSound->AttenuationSettings.bAttenuate)
	{
		return;
	}

	check(ActiveSound->AudioDevice);
	const FAudioDevice& AudioDevice = *ActiveSound->AudioDevice;
	const int32 ClosestListenerIndex = AudioDevice.FindClosestListenerIndex(ActiveSound->Transform);

	FAttenuationListenerData ListenerData = FAttenuationListenerData::Create(AudioDevice, ClosestListenerIndex, ActiveSound->Transform, ActiveSound->AttenuationSettings);
	ActiveSound->UpdateFocusData(DeltaTime, ListenerData);
}

bool FAudioVirtualLoop::Update(float DeltaTime, bool bForceUpdate)
{
	check(ActiveSound);

	// Keep playback time up-to-date as it may be used to evaluate whether or
	// not virtual sound is eligible for playback when compared against
	// actively playing sounds in concurrency checks.
	const float DeltaTimePitchCorrected = DeltaTime * ActiveSound->MinCurrentPitch;
	ActiveSound->PlaybackTime += DeltaTimePitchCorrected;
	TimeVirtualized += DeltaTimePitchCorrected;

#if UE_AUDIO_PROFILERTRACE_ENABLED
	IsVirtualizedPingTimeSec += DeltaTime;
#endif

	// Update playback time estimates for each wave instance for Seek Restart, factoring in pitch.
	// This set only has elements if the ActiveSound was set to Seek Restart upon initial virtualization.
	for (TPair<UPTRINT, float>& Pair : EffectivePlaybackTimes)
	{
		FWaveInstanceData& Data = WaveInstanceData[Pair.Key];
		Data.EffectivePitch = ActiveSound->GetPitch();

		if (USoundClass* SoundClass = Data.SoundClass)
		{
			// We must also walk through all the soundclass's parents, just in case
			while (SoundClass)
			{
				Data.EffectivePitch *= SoundClass->Properties.Pitch;
				SoundClass = SoundClass->ParentClass;
			}
		}
		float PitchModulationValue = 1.0f;
		if (Data.PitchModulationDestination.IsActive())
		{
			Data.PitchModulationDestination.ProcessControl(0.0f);
			PitchModulationValue = Audio::GetFrequencyMultiplier(Data.PitchModulationDestination.GetValue());
		}
		
		EffectivePlaybackTimes[Pair.Key] += DeltaTime * Data.EffectivePitch * PitchModulationValue;
	}

	const float UpdateDelta = TimeSinceLastUpdate + DeltaTime;
	if (bForceUpdate)
	{
		TimeSinceLastUpdate = 0.0f;
	}
	else if (UpdateInterval > 0.0f)
	{
		TimeSinceLastUpdate = UpdateDelta;
		if (UpdateInterval > TimeSinceLastUpdate)
		{
			return false;
		}
		TimeSinceLastUpdate = 0.0f;
	}

	if (!VolumeConcurrencyDestinations.IsEmpty())
	{
		float MaxModulationVolume = 0.0f;
		for (Audio::FModulationDestination& Destination : VolumeConcurrencyDestinations)
		{
			// ProcessControl() only takes values in unit space (dB)
			Destination.ProcessControl(0.0f);
			const float DestVolume = Destination.GetValue();
			if (DestVolume > MaxModulationVolume)
			{
				MaxModulationVolume = DestVolume;
			}
		}
		ActiveSound->MaxSourceModulationValue = MaxModulationVolume;
	}

#if ENABLE_AUDIO_DEBUG
	Audio::FAudioDebugger::DrawDebugInfo(*this);
#endif // ENABLE_AUDIO_DEBUG

#if UE_AUDIO_PROFILERTRACE_ENABLED
	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
	if (bChannelEnabled)
	{
		if (const FAudioDevice* AudioDevice = ActiveSound->AudioDevice)
		{
			const USoundBase* Sound = ActiveSound->GetSound();

			const FTransform& Transform = ActiveSound->Transform;
			const FVector& Location = Transform.GetLocation();
			const FRotator Rotator = Transform.GetRotation().Rotator();

			if (IsVirtualizedPingTimeSec >= IsVirtualizedPingWaitTimeSec)
			{
				UE_TRACE_LOG(Audio, VirtualLoopIsVirtualizedPing, AudioChannel)
					<< VirtualLoopIsVirtualizedPing.DeviceId(static_cast<uint32>(AudioDevice->DeviceID))
					<< VirtualLoopIsVirtualizedPing.Timestamp(FPlatformTime::Cycles64())
					<< VirtualLoopIsVirtualizedPing.PlayOrder(ActiveSound->GetPlayOrder())
					<< VirtualLoopIsVirtualizedPing.ComponentId(ActiveSound->GetAudioComponentID())
					<< VirtualLoopIsVirtualizedPing.Name(Sound ? *Sound->GetPathName() : TEXT("N/A"));

				IsVirtualizedPingTimeSec = 0.0f;
			}

			UE_TRACE_LOG(Audio, VirtualLoopUpdate, AudioChannel)
				<< VirtualLoopUpdate.DeviceId(static_cast<uint32>(AudioDevice->DeviceID))
				<< VirtualLoopUpdate.Timestamp(FPlatformTime::Cycles64())
				<< VirtualLoopUpdate.PlayOrder(ActiveSound->GetPlayOrder())
				<< VirtualLoopUpdate.TimeVirtualized(TimeVirtualized)
				<< VirtualLoopUpdate.PlaybackTime(ActiveSound->PlaybackTime)
				<< VirtualLoopUpdate.UpdateInterval(UpdateInterval)
				<< VirtualLoopUpdate.LocationX(Location.X)
				<< VirtualLoopUpdate.LocationY(Location.Y)
				<< VirtualLoopUpdate.LocationZ(Location.Z)
				<< VirtualLoopUpdate.RotatorPitch(Rotator.Pitch)
				<< VirtualLoopUpdate.RotatorYaw(Rotator.Yaw)
				<< VirtualLoopUpdate.RotatorRoll(Rotator.Roll);
		}
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	UpdateFocusData(UpdateDelta);

	// If not audible, update when will be checked again and return false
	if (!IsInAudibleRange(*ActiveSound))
	{
		CalculateUpdateInterval();
		return false;
	}

	return true;
}

#if UE_AUDIO_PROFILERTRACE_ENABLED
void FAudioVirtualLoop::OnTraceStarted()
{
	// Make sure we send virtual loop data to Audio Insights if it's opened in the middle of a PIE session
	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
	if (bChannelEnabled && ActiveSound)
	{
		check(ActiveSound);
		const USoundBase* Sound = ActiveSound->GetSound();
		check(Sound);

		if (const FAudioDevice* AudioDevice = ActiveSound->AudioDevice)
		{
			UE_TRACE_LOG(Audio, VirtualLoopVirtualize, AudioChannel)
				<< VirtualLoopVirtualize.DeviceId(static_cast<uint32>(AudioDevice->DeviceID))
				<< VirtualLoopVirtualize.Timestamp(FPlatformTime::Cycles64())
				<< VirtualLoopVirtualize.PlayOrder(ActiveSound->GetPlayOrder())
				<< VirtualLoopVirtualize.ComponentId(ActiveSound->GetAudioComponentID())
				<< VirtualLoopVirtualize.Name(Sound ? *Sound->GetPathName() : TEXT("N/A"));
		}
	}
}
#endif

bool FAudioVirtualLoop::ShouldListenerMoveForceUpdate(const FTransform& LastTransform, const FTransform& CurrentTransform)
{
	const float DistanceSq = FVector::DistSquared(LastTransform.GetTranslation(), CurrentTransform.GetTranslation());
	const float ForceUpdateDistSq = VirtualLoopsForceUpdateListenerMoveDistanceCVar * VirtualLoopsForceUpdateListenerMoveDistanceCVar;
	return DistanceSq > ForceUpdateDistSq;
}
