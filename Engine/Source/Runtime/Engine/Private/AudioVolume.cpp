// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AudioVolume.cpp: Used to affect audio settings in the game and editor.
=============================================================================*/

#include "Sound/AudioVolume.h"
#include "Engine/CollisionProfile.h"
#include "Engine/World.h"
#include "Sound/ReverbEffect.h"
#include "AudioDevice.h"
#include "Components/BrushComponent.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioVolume)

FInteriorSettings::FInteriorSettings()
	: bIsWorldSettings(false)
	, ExteriorVolume(1.0f)
	, ExteriorTime(0.5f)
	, ExteriorLPF(MAX_FILTER_FREQUENCY)
	, ExteriorLPFTime(0.5f)
	, InteriorVolume(1.0f)
	, InteriorTime(0.5f)
	, InteriorLPF(MAX_FILTER_FREQUENCY)
	, InteriorLPFTime(0.5f)
{
}

#if WITH_EDITORONLY_DATA
void FInteriorSettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_USE_LOW_PASS_FILTER_FREQ)
	{
		if (InteriorLPF > 0.0f && InteriorLPF < 1.0f)
		{
			float FilterConstant = 2.0f * FMath::Sin(UE_PI * 6000.0f * InteriorLPF / 48000);
			InteriorLPF = FilterConstant * MAX_FILTER_FREQUENCY;
		}

		if (ExteriorLPF > 0.0f && ExteriorLPF < 1.0f)
		{
			float FilterConstant = 2.0f * FMath::Sin(UE_PI * 6000.0f * ExteriorLPF / 48000);
			ExteriorLPF = FilterConstant * MAX_FILTER_FREQUENCY;
		}
	}
}
#endif

bool FInteriorSettings::operator==(const FInteriorSettings& Other) const
{
	return (Other.bIsWorldSettings == bIsWorldSettings)
		&& (Other.ExteriorVolume == ExteriorVolume) && (Other.ExteriorTime == ExteriorTime)
		&& (Other.ExteriorLPF == ExteriorLPF) && (Other.ExteriorLPFTime == ExteriorLPFTime)
		&& (Other.InteriorVolume == InteriorVolume) && (Other.InteriorTime == InteriorTime)
		&& (Other.InteriorLPF == InteriorLPF) && (Other.InteriorLPFTime == InteriorLPFTime);
}

bool FInteriorSettings::operator!=(const FInteriorSettings& Other) const
{
	return !(*this == Other);
}

AAudioVolume::AAudioVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	GetBrushComponent()->bAlwaysCreatePhysicsState = true;

	bColored = true;
	BrushColor = FColor(255, 255, 0, 255);

	bEnabled = true;
}

void AAudioVolume::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	DOREPLIFETIME(AAudioVolume, bEnabled);
}

FAudioVolumeProxy::FAudioVolumeProxy(const AAudioVolume* AudioVolume)
	: AudioVolumeID(AudioVolume->GetUniqueID())
	, WorldID(AudioVolume->GetWorld()->GetUniqueID())
	, Priority(AudioVolume->GetPriority())
	, ReverbSettings(AudioVolume->GetReverbSettings())
	, InteriorSettings(AudioVolume->GetInteriorSettings())
	, SubmixSendSettings(AudioVolume->GetSubmixSendSettings())
	, SubmixOverrideSettings(AudioVolume->GetSubmixOverrideSettings())
	, BodyInstance(AudioVolume->GetBrushComponent()->GetBodyInstance())
{
}

void AAudioVolume::AddProxy() const
{
	if (UWorld* World = GetWorld())
	{
		if (FAudioDeviceHandle AudioDeviceHandle = World->GetAudioDevice())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AddAudioVolumeProxy"), STAT_AudioAddAudioVolumeProxy, STATGROUP_TaskGraphTasks);

			FAudioVolumeProxy Proxy(this);

			FAudioThread::RunCommandOnAudioThread([AudioDeviceHandle, Proxy]() mutable
			{
				if (AudioDeviceHandle)
				{
					AudioDeviceHandle->AddAudioVolumeProxy(Proxy);
				}
			}, GET_STATID(STAT_AudioAddAudioVolumeProxy));
		}
	}
}

void FAudioDevice::AddAudioVolumeProxy(const FAudioVolumeProxy& Proxy)
{
	check(IsInAudioThread());

	AudioVolumeProxies.Add(Proxy.AudioVolumeID, Proxy);
	AudioVolumeProxies.ValueSort([](const FAudioVolumeProxy& A, const FAudioVolumeProxy& B) { return A.Priority > B.Priority; });

	InvalidateCachedInteriorVolumes();
}


void AAudioVolume::RemoveProxy() const
{
	// World will be NULL during exit purge.
	UWorld* World = GetWorld();
	if (World)
	{
		if (FAudioDeviceHandle AudioDeviceHandle = World->GetAudioDevice())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.RemoveAudioVolumeProxy"), STAT_AudioRemoveAudioVolumeProxy, STATGROUP_TaskGraphTasks);

			const uint32 AudioVolumeID = GetUniqueID();
			FAudioThread::RunCommandOnAudioThread([AudioDeviceHandle, AudioVolumeID]() mutable
			{
				if (AudioDeviceHandle)
				{
					AudioDeviceHandle->RemoveAudioVolumeProxy(AudioVolumeID);
				}
			}, GET_STATID(STAT_AudioRemoveAudioVolumeProxy));
		}
	}
}

void FAudioDevice::RemoveAudioVolumeProxy(const uint32 AudioVolumeID)
{
	check(IsInAudioThread());

	AudioVolumeProxies.Remove(AudioVolumeID);

	InvalidateCachedInteriorVolumes();
}

void AAudioVolume::UpdateProxy() const
{
	if (UWorld* World = GetWorld())
	{
		if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.UpdateAudioVolumeProxy"), STAT_AudioUpdateAudioVolumeProxy, STATGROUP_TaskGraphTasks);

			FAudioVolumeProxy Proxy(this);

			FAudioThread::RunCommandOnAudioThread([AudioDevice, Proxy]()
			{
				AudioDevice->UpdateAudioVolumeProxy(Proxy);
			}, GET_STATID(STAT_AudioUpdateAudioVolumeProxy));
		}
	}
}

void FAudioDevice::UpdateAudioVolumeProxy(const FAudioVolumeProxy& NewProxy)
{
	check(IsInAudioThread());

	if (FAudioVolumeProxy* CurrentProxy = AudioVolumeProxies.Find(NewProxy.AudioVolumeID))
	{
		const float CurrentPriority = CurrentProxy->Priority;

		*CurrentProxy = NewProxy;

		// Flag that the proxy changed so it can propagate any changes to runtime systems
		CurrentProxy->bChanged = true;

		if (CurrentPriority != NewProxy.Priority)
		{
			AudioVolumeProxies.ValueSort([](const FAudioVolumeProxy& A, const FAudioVolumeProxy& B) { return A.Priority > B.Priority; });
		}
	}
}

void AAudioVolume::PostUnregisterAllComponents()
{
	// Route clear to super first.
	Super::PostUnregisterAllComponents();

	// Component can be nulled due to GC at this point
	if (GetRootComponent())
	{
		GetRootComponent()->TransformUpdated.RemoveAll(this);
	}
	RemoveProxy();

	if (UWorld* World = GetWorld())
	{
		World->AudioVolumes.Remove(this);
	}
}

void AAudioVolume::PostRegisterAllComponents()
{
	// Route update to super first.
	Super::PostRegisterAllComponents();

	GetRootComponent()->TransformUpdated.AddUObject(this, &AAudioVolume::TransformUpdated);
	if (bEnabled)
	{
		AddProxy();
	}

	UWorld* World = GetWorld();
	World->AudioVolumes.Add(this);
	World->AudioVolumes.Sort([](const AAudioVolume& A, const AAudioVolume& B) { return (A.GetPriority() > B.GetPriority()); });
}

void AAudioVolume::TransformUpdated(USceneComponent* InRootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	UpdateProxy();
}

void AAudioVolume::SetEnabled(const bool bNewEnabled)
{
	if (bNewEnabled != bEnabled)
	{
		bEnabled = bNewEnabled;
		if (bEnabled)
		{
			AddProxy();
		}
		else
		{
			RemoveProxy();
		}
	}
}

void AAudioVolume::OnRep_bEnabled()
{
	if (bEnabled)
	{
		AddProxy();
	}
	else
	{
		RemoveProxy();
	}
}

void AAudioVolume::SetPriority(const float NewPriority)
{
	if (NewPriority != Priority)
	{
		Priority = NewPriority;
		if (UWorld* World = GetWorld())
		{
			World->AudioVolumes.Sort([](const AAudioVolume& A, const AAudioVolume& B) { return (A.GetPriority() > B.GetPriority()); });
			if (bEnabled)
			{
				UpdateProxy();
			}
		}
	}
}

void AAudioVolume::SetInteriorSettings(const FInteriorSettings& NewInteriorSettings)
{
	if (NewInteriorSettings != AmbientZoneSettings)
	{
		AmbientZoneSettings = NewInteriorSettings;
		if (bEnabled)
		{
			UpdateProxy();
		}
	}
}

void AAudioVolume::SetSubmixSendSettings(const TArray<FAudioVolumeSubmixSendSettings>& NewSubmixSendSettings)
{
	SubmixSendSettings = NewSubmixSendSettings;
	if (bEnabled)
	{
		UpdateProxy();
	}
}

void AAudioVolume::SetSubmixOverrideSettings(const TArray<FAudioVolumeSubmixOverrideSettings>& NewSubmixOverrideSettings)
{
	SubmixOverrideSettings = NewSubmixOverrideSettings;
	if (bEnabled)
	{
		UpdateProxy();
	}
}

void AAudioVolume::SetReverbSettings(const FReverbSettings& NewReverbSettings)
{
	if (NewReverbSettings != Settings)
	{
		Settings = NewReverbSettings;
		if (bEnabled)
		{
			UpdateProxy();
		}
	}
}


#if WITH_EDITOR
void AAudioVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	Settings.Volume = FMath::Clamp<float>( Settings.Volume, 0.0f, 1.0f );
	AmbientZoneSettings.InteriorTime = FMath::Max<float>( 0.01f, AmbientZoneSettings.InteriorTime );
	AmbientZoneSettings.InteriorLPFTime = FMath::Max<float>( 0.01f, AmbientZoneSettings.InteriorLPFTime );
	AmbientZoneSettings.ExteriorTime = FMath::Max<float>( 0.01f, AmbientZoneSettings.ExteriorTime );
	AmbientZoneSettings.ExteriorLPFTime = FMath::Max<float>( 0.01f, AmbientZoneSettings.ExteriorLPFTime );

	if (PropertyChangedEvent.Property)
	{
		static FName NAME_Priority = GET_MEMBER_NAME_CHECKED(AAudioVolume, Priority);
		static FName NAME_Enabled = GET_MEMBER_NAME_CHECKED(AAudioVolume, bEnabled);
		static FName NAME_ApplyReverb = GET_MEMBER_NAME_CHECKED(FReverbSettings, bApplyReverb);

		FName PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == NAME_Priority)
		{
			if (UWorld* World = GetWorld())
			{
				World->AudioVolumes.Sort([](const AAudioVolume& A, const AAudioVolume& B) { return (A.GetPriority() > B.GetPriority()); });
			}
		}
		else if (PropertyName == NAME_Enabled)
		{
			if (bEnabled)
			{
				AddProxy();
			}
			else
			{
				RemoveProxy();
			}
		}
		else if (PropertyName == NAME_ApplyReverb)
		{
			if (Settings.ReverbEffect)
			{
				Settings.ReverbEffect->bChanged = true;
			}
		}

		if (bEnabled)
		{
			UpdateProxy();
		}
	}


}
#endif // WITH_EDITOR

