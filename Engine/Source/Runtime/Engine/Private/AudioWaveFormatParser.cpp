// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWaveFormatParser.h"
#include "Audio.h"

// AT9 is an "extensible" type wave format

// Chunk IDs and Type identifiers
#define CHUNK_ID_RIFF			(0x46464952)	// "RIFF"
#define CHUNK_TYPE_WAVE			(0x45564157)	// "WAVE"
#define CHUNK_ID_FMT			(0x20746D66)	// "fmt "
#define CHUNK_ID_FACT			(0x74636166)	// "fact"
#define CHUNK_ID_DATA			(0x61746164)	// "data"
#define CHUNK_ID_SMPL			(0x6C706D73)	// "smpl"
#define STREAM_LOOPINFO_MAX		(2)


bool ParseWaveFormatHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FWaveFormatInfo& OutHeader)
{
	if (!InSrcBufferData || InSrcBufferDataSize < sizeof(OutHeader.RiffWaveHeader))
	{
		UE_LOGF(LogAudio, Error, "Failed to parse wave format header. Buffer data was null or empty.");
		return false;
	}

	OutHeader.DataStartOffset = 0;

	uint32 CurrByte = 0;
	uint32 CurrentChunkDataSize = 0;

	// first get the RIFF chunk to make sure we have the correct file type
	FMemory::Memcpy(&OutHeader.RiffWaveHeader, &InSrcBufferData[CurrByte], sizeof(OutHeader.RiffWaveHeader));

	CurrByte += sizeof(OutHeader.RiffWaveHeader);

	// Check for "RIFF" in the ChunkID
	if (OutHeader.RiffWaveHeader.ChunkId != CHUNK_ID_RIFF)
	{
		UE_LOGF(LogAudio, Error, "Beginning of wave file was not \"RIFF\"");
		return false;
	}

	// Check to see if we've found the "WAVE" chunk (apparently there could be more than one "RIFF" chunk?)
	if (OutHeader.RiffWaveHeader.TypeId != CHUNK_TYPE_WAVE)
	{
		UE_LOGF(LogAudio, Error, "First wave RIFF chunk is not a \"WAVE\" type");
		return false;
	}

	// Now read the other chunk headers to get file information
	while (CurrByte < InSrcBufferDataSize)
	{
		// Read the next chunk header
		FChunkHeader ChunkHeader;
		FMemory::Memcpy(&ChunkHeader, &InSrcBufferData[CurrByte], sizeof(FChunkHeader));

		// Offset the byte index by the sizeof chunk header
		CurrByte += sizeof(FChunkHeader);

		// Now read which type of chunk this is and get the header info
		switch (ChunkHeader.ChunkId)
		{
			case CHUNK_ID_FMT:
			{
				OutHeader.FmtChunkHeader = ChunkHeader;

				FMemory::Memcpy(&OutHeader.FmtChunk, &InSrcBufferData[CurrByte], sizeof(FFormatChunk));

				CurrByte += sizeof(FFormatChunk);

				// The rest of the data in this chunk is unknown so skip it
				CurrentChunkDataSize = (CurrentChunkDataSize < sizeof(FFormatChunk)) ? 0 : CurrentChunkDataSize - sizeof(FFormatChunk);
			}
			break;

			case CHUNK_ID_FACT:
			{
				OutHeader.FactChunkHeader = ChunkHeader;
				FMemory::Memcpy(&OutHeader.FactChunk, &InSrcBufferData[CurrByte], sizeof(FFactChunk));

				CurrByte += sizeof(FFactChunk);

				// The rest of the data in this chunk is unknown so skip it
				CurrentChunkDataSize = (CurrentChunkDataSize < sizeof(FFactChunk)) ? 0 : CurrentChunkDataSize - sizeof(FFactChunk);
			}
			break;

			case CHUNK_ID_DATA:
			{
				OutHeader.DataChunkHeader = ChunkHeader;

				// This is where bit-stream data starts in the AT9 file
				OutHeader.DataStartOffset = CurrByte;

				CurrentChunkDataSize = OutHeader.DataChunkHeader.ChunkDataSize;
			}
			break;

			case CHUNK_ID_SMPL:
			{
				OutHeader.SampleChunkHeader = ChunkHeader;

				uint32 SampleChunkSize = sizeof(FSampleChunk) - sizeof(FSampleLoop)* STREAM_LOOPINFO_MAX;

				FMemory::Memcpy(&OutHeader.SampleChunk, &InSrcBufferData[CurrByte], SampleChunkSize);

				CurrByte += SampleChunkSize;

				// Read loop information
				uint32 ReadSize = 0;
				if (OutHeader.SampleChunk.SampleLoops <= STREAM_LOOPINFO_MAX)
				{
					ReadSize = sizeof(FSampleLoop)* OutHeader.SampleChunk.SampleLoops;
					FMemory::Memcpy(&OutHeader.SampleChunk.SampleLoop, &InSrcBufferData[CurrByte], ReadSize);
				}
				else
				{
					ReadSize = sizeof(FSampleLoop)* STREAM_LOOPINFO_MAX;
					FMemory::Memcpy(&OutHeader.SampleChunk.SampleLoop, &InSrcBufferData[CurrByte], ReadSize);
				}

				CurrByte += ReadSize;

				CurrentChunkDataSize = (CurrentChunkDataSize < sizeof(FSampleChunk)) ? 0 : CurrentChunkDataSize - sizeof(FSampleChunk);
			}
			break;

			default:
			{
				UE_LOGF(LogAudio, Warning, "Wave file contained unknown RIFF chunk type (%d)", ChunkHeader.ChunkId);
			}
			break;

		}

		// Offset the byte read index by the current chunk's data size
		CurrByte += CurrentChunkDataSize;
	}

	return true;
}
