// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/Actor.h"
#include "Math/RotationMatrix.h"


FBasedPosition::FBasedPosition()
	: Base(NULL)
	, Position(ForceInit)
	, CachedBaseLocation(ForceInit)
	, CachedBaseRotation(ForceInit)
	, CachedTransPosition(ForceInit)
{
}

FBasedPosition::FBasedPosition( class AActor *InBase, const FVector& InPosition )
{
	Set( InBase, InPosition );
}

FArchive& operator<<( FArchive& Ar, FBasedPosition& T )
{
	Ar << T.Base;
	Ar << T.Position;	
	return Ar;
}

// Retrieve world location of this position
FVector FBasedPosition::operator*() const
{
	if( Base )
	{
		const FVector BaseLocation = Base->GetActorLocation();
		const FRotator BaseRotation = Base->GetActorRotation();

		// If base hasn't changed location/rotation use cached transformed position
		if( (CachedBaseLocation != BaseLocation) ||	(CachedBaseRotation != BaseRotation) )
		{
			CachedBaseLocation	= BaseLocation;
			CachedBaseRotation	= BaseRotation;
			CachedTransPosition = BaseLocation + FRotationMatrix(BaseRotation).TransformPosition(Position);
		}

		return CachedTransPosition;
	}
	return Position;
}

void FBasedPosition::Set(class AActor* InBase, const FVector& InPosition)
{
	if( InPosition.IsNearlyZero() )
	{
		Base = NULL;
		Position = FVector::ZeroVector;
		return;
	}

	Base = (InBase && InBase->GetRootComponent() && (InBase->GetRootComponent()->Mobility != EComponentMobility::Static)) ? InBase : NULL;
	if( Base )
	{
		const FVector BaseLocation = Base->GetActorLocation();
		const FRotator BaseRotation = Base->GetActorRotation();

		CachedBaseLocation	= BaseLocation;
		CachedBaseRotation	= BaseRotation;
		CachedTransPosition = InPosition;
		Position = FTransform(BaseRotation).InverseTransformPosition(InPosition - BaseLocation);
	}
	else
	{
		Position = InPosition;
	}
}

void FBasedPosition::Clear()
{
	Base = NULL;
	Position = FVector::ZeroVector;
}
