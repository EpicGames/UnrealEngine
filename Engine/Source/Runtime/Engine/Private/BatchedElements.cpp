// Copyright Epic Games, Inc. All Rights Reserved.


#include "BatchedElements.h"

#include "SceneView.h"
#include "SimpleElementShaders.h"
#include "RHIStaticStates.h"
#include "MeshPassProcessor.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "PipelineStateCache.h"
#include "SceneRelativeViewMatrices.h"
#include "ShaderParameterUtils.h"
#include "GlobalRenderResources.h"
#include "HDRHelper.h"
#include "RHIResourceUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogBatchedElements, Log, All);

FSimpleElementVertex::FSimpleElementVertex() = default;

FSimpleElementVertex::FSimpleElementVertex(const FVector4f& InPosition, const FVector2f& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor)
	: Position(InPosition, FVector4f{0.0f, 0.0f, 0.0f, 0.0f})
	, TextureCoordinate(InTextureCoordinate)
	, Color(InColor)
	, HitProxyIdColor(InHitProxyColor)
{
}

FSimpleElementVertex::FSimpleElementVertex(const FVector4f& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor)
	: Position(InPosition, FVector4f{0.0f, 0.0f, 0.0f, 0.0f})
	, TextureCoordinate(InTextureCoordinate)
	, Color(InColor)
	, HitProxyIdColor(InHitProxyColor)
{
}

FSimpleElementVertex::FSimpleElementVertex(const FVector3f& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor)
	: Position(FVector4f{ InPosition, 1.0 }, FVector4f{0.0f, 0.0f, 0.0f, 0.0f})
	, TextureCoordinate(FVector2f(InTextureCoordinate))
	, Color(InColor)
	, HitProxyIdColor(InHitProxyColor)
{
}

FSimpleElementVertex::FSimpleElementVertex(const FVector4d& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor)
	: Position(InPosition)
	, TextureCoordinate(InTextureCoordinate)
	, Color(InColor)
	, HitProxyIdColor(InHitProxyColor)
{}

FSimpleElementVertex::FSimpleElementVertex(const FVector3d& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor)
	: Position(InPosition, 1.0)
	, TextureCoordinate(InTextureCoordinate)
	, Color(InColor)
	, HitProxyIdColor(InHitProxyColor)
{}

FSimpleElementVertex::FSimpleElementVertex(const FVector4f& InPosition, const FVector2f& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId)
	: FSimpleElementVertex(InPosition, InTextureCoordinate, InColor, InHitProxyId.GetColor())
{
}

FSimpleElementVertex::FSimpleElementVertex(const FVector4f& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId)
	: FSimpleElementVertex(InPosition, InTextureCoordinate, InColor, InHitProxyId.GetColor())
{
}

FSimpleElementVertex::FSimpleElementVertex(const FVector3f& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId)
	: FSimpleElementVertex(InPosition, InTextureCoordinate, InColor, InHitProxyId.GetColor())
{
}

FSimpleElementVertex::FSimpleElementVertex(const FVector4d& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId)
	: FSimpleElementVertex(InPosition, InTextureCoordinate, InColor, InHitProxyId.GetColor())
{
}

FSimpleElementVertex::FSimpleElementVertex(const FVector3d& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId)
	: FSimpleElementVertex(InPosition, InTextureCoordinate, InColor, InHitProxyId.GetColor())
{
}

FSimpleElementVertexDeclaration::FSimpleElementVertexDeclaration() = default;
FSimpleElementVertexDeclaration::FSimpleElementVertexDeclaration(FSimpleElementVertexDeclaration&&) = default;
FSimpleElementVertexDeclaration::~FSimpleElementVertexDeclaration() = default;

void FSimpleElementVertexDeclaration::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexDeclarationElementList Elements;
	uint16 Stride = sizeof(FSimpleElementVertex);
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSimpleElementVertex, Position.High), VET_Float4, 0, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSimpleElementVertex, Position.Low), VET_Float4, 1, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSimpleElementVertex, TextureCoordinate), VET_Float2, 2, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSimpleElementVertex, Color), VET_Float4, 3, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSimpleElementVertex, HitProxyIdColor), VET_Color, 4, Stride));
	VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
}

void FSimpleElementVertexDeclaration::ReleaseRHI()
{
	VertexDeclarationRHI.SafeRelease();
}

/** The simple element vertex declaration. */
TGlobalResource<FSimpleElementVertexDeclaration> GSimpleElementVertexDeclaration;

EBlendModeFilter::Type GetBlendModeFilter(ESimpleElementBlendMode BlendMode)
{
	if (BlendMode == SE_BLEND_Opaque || BlendMode == SE_BLEND_Masked || BlendMode == SE_BLEND_MaskedDistanceField || BlendMode == SE_BLEND_MaskedDistanceFieldShadowed)
	{
		return EBlendModeFilter::OpaqueAndMasked;
	}
	else
	{
		return EBlendModeFilter::Translucent;
	}
}

FBatchedElements::FBatchedElements()
	: WireTriVerts(/*InNeedsCPUAccess*/true) // Keep vertices on buffer creation
	, MaxMeshIndicesAllowed(GDrawUPIndexCheckCount / sizeof(int32))
	// the index buffer is 2 bytes, so make sure we only address 0xFFFF vertices in the index buffer
	, MaxMeshVerticesAllowed(FMath::Min<uint32>(0xFFFF, GDrawUPVertexCheckCount / sizeof(FSimpleElementVertex)))
{
}

void FBatchedElements::AddLine(const FVector& Start, const FVector& End, const FLinearColor& Color, FHitProxyId HitProxyId, float Thickness, float DepthBias, bool bScreenSpace)
{
	// Ensure the line isn't masked out.  Some legacy code relies on Color.A being ignored.
	FLinearColor OpaqueColor(Color);
	OpaqueColor.A = 1;

	if (Thickness == 0.0f)
	{
		if (DepthBias == 0.0f)
		{
			LineVertices.Emplace(Start, FVector2D::ZeroVector, OpaqueColor, HitProxyId);
			LineVertices.Emplace(End, FVector2D::ZeroVector, OpaqueColor, HitProxyId);
		}
		else
		{
			// Draw degenerate triangles in wireframe mode to support depth bias (d3d11 and opengl3 don't support depth bias on line primitives, but do on wireframes)
			FBatchedWireTris& WireTri = WireTris.AddDefaulted_GetRef();
			WireTri.DepthBias = DepthBias;
			WireTriVerts.Emplace(Start, FVector2D::ZeroVector, OpaqueColor, HitProxyId);
			WireTriVerts.Emplace(End, FVector2D::ZeroVector, OpaqueColor, HitProxyId);
			WireTriVerts.Emplace(End, FVector2D::ZeroVector, OpaqueColor, HitProxyId);
		}
	}
	else
	{
		FBatchedThickLines& ThickLine = ThickLines.AddDefaulted_GetRef();
		ThickLine.Start = Start;
		ThickLine.End = End;
		ThickLine.Thickness = Thickness;
		ThickLine.Color = OpaqueColor;
		ThickLine.HitProxyColor = HitProxyId.GetColor();
		ThickLine.DepthBias = DepthBias;
		ThickLine.bScreenSpace = bScreenSpace;
	}
}

void FBatchedElements::AddTranslucentLine(const FVector& Start, const FVector& End, const FLinearColor& Color, FHitProxyId HitProxyId, float Thickness, float DepthBias, bool bScreenSpace)
{
	if (Thickness == 0.0f)
	{
		if (DepthBias == 0.0f)
		{
			LineVertices.Emplace(Start, FVector2D::ZeroVector, Color, HitProxyId);
			LineVertices.Emplace(End, FVector2D::ZeroVector, Color, HitProxyId);
		}
		else
		{
			// Draw degenerate triangles in wireframe mode to support depth bias (d3d11 and opengl3 don't support depth bias on line primitives, but do on wireframes)
			FBatchedWireTris& WireTri = WireTris.AddDefaulted_GetRef();
			WireTri.DepthBias = DepthBias;
			WireTriVerts.Emplace(Start, FVector2D::ZeroVector, Color, HitProxyId);
			WireTriVerts.Emplace(End, FVector2D::ZeroVector, Color, HitProxyId);
			WireTriVerts.Emplace(End, FVector2D::ZeroVector, Color, HitProxyId);
		}
	}
	else
	{
		FBatchedThickLines& ThickLine = ThickLines.AddDefaulted_GetRef();
		ThickLine.Start = Start;
		ThickLine.End = End;
		ThickLine.Thickness = Thickness;
		ThickLine.Color = Color;
		ThickLine.HitProxyColor = HitProxyId.GetColor();
		ThickLine.DepthBias = DepthBias;
		ThickLine.bScreenSpace = bScreenSpace;

	}
}

void FBatchedElements::AddPoint(const FVector& Position,float Size,const FLinearColor& Color,FHitProxyId HitProxyId)
{
	// Ensure the point isn't masked out.  Some legacy code relies on Color.A being ignored.
	FLinearColor OpaqueColor(Color);
	OpaqueColor.A = 1;

	FBatchedPoint& Point = Points.AddDefaulted_GetRef();
	Point.Position = Position;
	Point.Size = Size;
	Point.Color = OpaqueColor.ToFColor(true);
	Point.HitProxyColor = HitProxyId.GetColor();
}

int32 FBatchedElements::AddVertex(const FVector4& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId HitProxyId)
{
	int32 VertexIndex = MeshVertices.Num();
	MeshVertices.Emplace(InPosition, InTextureCoordinate, InColor, HitProxyId);
	return VertexIndex;
}

int32 FBatchedElements::AddVertexf(const FVector4f& InPosition,const FVector2f& InTextureCoordinate,const FLinearColor& InColor,FHitProxyId HitProxyId)
{
	int32 VertexIndex = MeshVertices.Num();
	MeshVertices.Emplace(InPosition,InTextureCoordinate,InColor,HitProxyId);
	return VertexIndex;
}

/** Adds a triangle to the batch. */
void FBatchedElements::AddTriangle(int32 V0,int32 V1,int32 V2,const FTexture* Texture,EBlendMode BlendMode)
{
	ESimpleElementBlendMode SimpleElementBlendMode = SE_BLEND_Opaque;
	switch (BlendMode)
	{
	case BLEND_Opaque:
		SimpleElementBlendMode = SE_BLEND_Opaque; 
		break;
	case BLEND_Masked:
	case BLEND_Translucent:
		SimpleElementBlendMode = SE_BLEND_Translucent; 
		break;
	case BLEND_Additive:
		SimpleElementBlendMode = SE_BLEND_Additive; 
		break;
	case BLEND_Modulate:
		SimpleElementBlendMode = SE_BLEND_Modulate; 
		break;
	case BLEND_AlphaComposite:
		SimpleElementBlendMode = SE_BLEND_AlphaComposite;
		break;
	case BLEND_AlphaHoldout:
		SimpleElementBlendMode = SE_BLEND_AlphaHoldout;
		break;
	};
	AddTriangle(V0,V1,V2,Texture,SimpleElementBlendMode);
}

void FBatchedElements::AddTriangle(int32 V0, int32 V1, int32 V2, const FTexture* Texture, ESimpleElementBlendMode BlendMode, const FDepthFieldGlowInfo& GlowInfo)
{
	AddTriangleExtensive( V0, V1, V2, NULL, Texture, BlendMode, GlowInfo );
}

	
void FBatchedElements::AddTriangle(int32 V0,int32 V1,int32 V2,FBatchedElementParameters* BatchedElementParameters,ESimpleElementBlendMode BlendMode)
{
	AddTriangleExtensive( V0, V1, V2, BatchedElementParameters, GWhiteTexture, BlendMode );
}


void FBatchedElements::AddTriangleExtensive(int32 V0,int32 V1,int32 V2,FBatchedElementParameters* BatchedElementParameters,const FTexture* Texture,ESimpleElementBlendMode BlendMode, const FDepthFieldGlowInfo& GlowInfo)
{
	check(Texture);

	// Find an existing mesh element for the given texture and blend mode
	FBatchedMeshElement* MeshElement = NULL;
	for(int32 MeshIndex = 0;MeshIndex < MeshElements.Num();MeshIndex++)
	{
		const FBatchedMeshElement& CurMeshElement = MeshElements[MeshIndex];
		if( CurMeshElement.Texture == Texture && 
			CurMeshElement.BatchedElementParameters.GetReference() == BatchedElementParameters &&
			CurMeshElement.BlendMode == BlendMode &&
			// make sure we are not overflowing on indices
			(CurMeshElement.Indices.Num()+3) < MaxMeshIndicesAllowed &&
			CurMeshElement.GlowInfo == GlowInfo )
		{
			// make sure we are not overflowing on vertices
			int32 DeltaV0 = (V0 - (int32)CurMeshElement.MinVertex);
			int32 DeltaV1 = (V1 - (int32)CurMeshElement.MinVertex);
			int32 DeltaV2 = (V2 - (int32)CurMeshElement.MinVertex);
			if( DeltaV0 >= 0 && DeltaV0 < MaxMeshVerticesAllowed &&
				DeltaV1 >= 0 && DeltaV1 < MaxMeshVerticesAllowed &&
				DeltaV2 >= 0 && DeltaV2 < MaxMeshVerticesAllowed )
			{
				MeshElement = &MeshElements[MeshIndex];
				break;
			}			
		}
	}
	if(!MeshElement)
	{
		// make sure that vertex indices are close enough to fit within MaxVerticesAllowed
		if( FMath::Abs(V0 - V1) >= MaxMeshVerticesAllowed ||
			FMath::Abs(V0 - V2) >= MaxMeshVerticesAllowed )
		{
			UE_LOGF(LogBatchedElements, Warning, "Omitting FBatchedElements::AddTriangle due to sparce vertices V0=%i,V1=%i,V2=%i",V0,V1,V2);
		}
		else
		{
			// Create a new mesh element for the texture if this is the first triangle encountered using it.
			MeshElement = &MeshElements.AddDefaulted_GetRef();
			MeshElement->Texture = Texture;
			MeshElement->BatchedElementParameters = BatchedElementParameters;
			MeshElement->BlendMode = BlendMode;
			MeshElement->GlowInfo = GlowInfo;
			MeshElement->MaxVertex = V0;
			// keep track of the min vertex index used
			MeshElement->MinVertex = FMath::Min(FMath::Min(V0,V1),V2);
		}
	}

	if( MeshElement )
	{
		// Add the triangle's indices to the mesh element's index array.
		MeshElement->Indices.Add(V0 - MeshElement->MinVertex);
		MeshElement->Indices.Add(V1 - MeshElement->MinVertex);
		MeshElement->Indices.Add(V2 - MeshElement->MinVertex);

		// keep track of max vertex used in this mesh batch
		MeshElement->MaxVertex = FMath::Max(FMath::Max(FMath::Max(V0,(int32)MeshElement->MaxVertex),V1),V2);
	}
}

/** 
* Reserves space in mesh vertex array
* 
* @param NumMeshVerts - number of verts to reserve space for
* @param Texture - used to find the mesh element entry
* @param BlendMode - used to find the mesh element entry
*/
void FBatchedElements::AddReserveVertices(int32 NumMeshVerts)
{
	MeshVertices.Reserve( MeshVertices.Num() + NumMeshVerts );
}

void FBatchedElements::ReserveVertices(int32 NumMeshVerts)
{
	MeshVertices.Reserve( NumMeshVerts );
}

/** 
 * Reserves space in line vertex array
 * 
 * @param NumLines - number of lines to reserve space for
 * @param bDepthBiased - whether reserving depth-biased lines or non-biased lines
 * @param bThickLines - whether reserving regular lines or thick lines
 */
void FBatchedElements::AddReserveLines(int32 NumLines, bool bDepthBiased, bool bThickLines)
{
	if (!bThickLines)
	{
		if (!bDepthBiased)
		{
			LineVertices.Reserve(LineVertices.Num() + NumLines * 2);
		}
		else
		{
			WireTris.Reserve(WireTris.Num() + NumLines);
			WireTriVerts.Reserve(WireTriVerts.Num() + NumLines * 3);
		}
	}
	else
	{
		ThickLines.Reserve(ThickLines.Num() + NumLines);
	}
}

/**
* Reserves space in triangle arrays 
* 
* @param NumMeshTriangles - number of triangles to reserve space for
* @param Texture - used to find the mesh element entry
* @param BlendMode - used to find the mesh element entry
*/
void FBatchedElements::AddReserveTriangles(int32 NumMeshTriangles,const FTexture* Texture,ESimpleElementBlendMode BlendMode)
{
	for(int32 MeshIndex = 0;MeshIndex < MeshElements.Num();MeshIndex++)
	{
		FBatchedMeshElement& CurMeshElement = MeshElements[MeshIndex];
		if( CurMeshElement.Texture == Texture && 
			CurMeshElement.BatchedElementParameters.GetReference() == NULL &&
			CurMeshElement.BlendMode == BlendMode &&
			(CurMeshElement.Indices.Num()+3) < MaxMeshIndicesAllowed )
		{
			CurMeshElement.Indices.Reserve( CurMeshElement.Indices.Num() + NumMeshTriangles );
			break;
		}
	}	
}

void FBatchedElements::ReserveTriangles(int32 NumMeshTriangles, const FTexture* Texture, ESimpleElementBlendMode BlendMode)
{
	for (int32 MeshIndex = 0; MeshIndex < MeshElements.Num(); MeshIndex++)
	{
		FBatchedMeshElement& CurMeshElement = MeshElements[MeshIndex];
		if (CurMeshElement.Texture == Texture &&
		   CurMeshElement.BatchedElementParameters.GetReference() == NULL &&
		   CurMeshElement.BlendMode == BlendMode &&
		   (CurMeshElement.Indices.Num()+3) < MaxMeshIndicesAllowed)
		{
			CurMeshElement.Indices.Reserve( NumMeshTriangles );
			break;
		}
	}
}

void FBatchedElements::AddSprite(
	const FVector& Position,
	float SizeX,
	float SizeY,
	const FTexture* Texture,
	const FLinearColor& Color,
	FHitProxyId HitProxyId,
	float U,
	float UL,
	float V,
	float VL,
	uint8 BlendMode,
	float OpacityMaskRefVal
	)
{
	check(Texture);

	FBatchedSprite& Sprite = Sprites.AddDefaulted_GetRef();
	Sprite.Position = Position;
	Sprite.SizeX = SizeX;
	Sprite.SizeY = SizeY;
	Sprite.Texture = Texture;
	Sprite.Color = Color;
	Sprite.HitProxyColor = HitProxyId.GetColor();
	Sprite.U = U;
	Sprite.UL = UL == 0.f ? Texture->GetSizeX() : UL;
	Sprite.V = V;
	Sprite.VL = VL == 0.f ? Texture->GetSizeY() : VL;
	Sprite.OpacityMaskRefVal = OpacityMaskRefVal;
	Sprite.BlendMode = BlendMode;
}

/** Translates a ESimpleElementBlendMode into a RHI state change for rendering a mesh with the blend mode normally. */
static void SetBlendState(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ESimpleElementBlendMode BlendMode)
{
	// Override blending operations to accumulate alpha
	static const auto CVarCompositeMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.UI.CompositeMode"));

	const bool bCompositeUI = GRHISupportsHDROutput
		&& CVarCompositeMode && CVarCompositeMode->GetValueOnRenderThread() != 0 
		&& IsHDREnabled();

	if (bCompositeUI)
	{
		// Compositing to offscreen buffer, so alpha needs to be accumulated in a sensible manner
		switch (BlendMode)
		{
		case SE_BLEND_Translucent:
		case SE_BLEND_TranslucentDistanceField:
		case SE_BLEND_TranslucentDistanceFieldShadowed:
		case SE_BLEND_TranslucentAlphaOnly:
			BlendMode = SE_BLEND_AlphaBlend;
			break;

		default:
			// Blend mode is reasonable as-is
			break;
		};
	}

	switch(BlendMode)
	{
	case SE_BLEND_Opaque:
	case SE_BLEND_Masked:
	case SE_BLEND_MaskedDistanceField:
	case SE_BLEND_MaskedDistanceFieldShadowed:
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		break;
	case SE_BLEND_Translucent:
	case SE_BLEND_TranslucentDistanceField:
	case SE_BLEND_TranslucentDistanceFieldShadowed:
	case SE_BLEND_TranslucentAlphaOnly:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
		break;
	case SE_BLEND_Additive:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
		break;
	case SE_BLEND_Modulate:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero>::GetRHI();
		break;
	case SE_BLEND_AlphaComposite:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI();
		break;
	case SE_BLEND_AlphaHoldout:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
		break;
	case SE_BLEND_TranslucentAlphaOnlyWriteAlpha:
	case SE_BLEND_AlphaBlend:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI();
		break;
	case SE_BLEND_RGBA_MASK_END:
	case SE_BLEND_RGBA_MASK_START:
		break;
	}
}

/** Translates a ESimpleElementBlendMode into a RHI state change for rendering a mesh with the blend mode for hit testing. */
static void SetHitTestingBlendState(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ESimpleElementBlendMode BlendMode)
{
	switch(BlendMode)
	{
	case SE_BLEND_Opaque:
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		break;
	case SE_BLEND_Masked:
	case SE_BLEND_MaskedDistanceField:
	case SE_BLEND_MaskedDistanceFieldShadowed:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		break;
	case SE_BLEND_AlphaComposite:
	case SE_BLEND_AlphaHoldout:
	case SE_BLEND_AlphaBlend:
	case SE_BLEND_Translucent:
	case SE_BLEND_TranslucentDistanceField:
	case SE_BLEND_TranslucentDistanceFieldShadowed:
	case SE_BLEND_TranslucentAlphaOnly:
	case SE_BLEND_TranslucentAlphaOnlyWriteAlpha:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		break;
	case SE_BLEND_Additive:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		break;
	case SE_BLEND_Modulate:
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		break;
	case SE_BLEND_RGBA_MASK_END:
	case SE_BLEND_RGBA_MASK_START:
		break;
	}
}

/** Global smoothing width for rendering batched elements with distance field blend modes */
float GBatchedElementSmoothWidth = 4;


FAutoConsoleVariableRef CVarWellCanvasDistanceFieldSmoothWidth(TEXT("Canvas.DistanceFieldSmoothness"), GBatchedElementSmoothWidth, TEXT("Global sharpness of distance field fonts/shapes rendered by canvas."), ECVF_Default);

template<class TSimpleElementPixelShader>
static TShaderRef<TSimpleElementPixelShader> GetPixelShader(ESimpleElementBlendMode BlendMode, ERHIFeatureLevel::Type FeatureLevel)
{
	return TShaderMapRef<TSimpleElementPixelShader>(GetGlobalShaderMap(FeatureLevel));
}

/**
 * Sets the appropriate vertex and pixel shader.
 */
void FBatchedElements::PrepareShaders(
	FRHICommandList& RHICmdList,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	uint32 StencilRef,
	ERHIFeatureLevel::Type FeatureLevel,
	ESimpleElementBlendMode BlendMode,
	const FDFRelativeViewMatrices& ViewMatrices,
	FBatchedElementParameters* BatchedElementParameters,
	const FTexture* Texture,
	bool bHitTesting,
	float Gamma,
	const FDepthFieldGlowInfo* GlowInfo,
	const FSceneView* View,
	float OpacityMaskRefVal
	) const
{
	// used to mask individual channels and desaturate
	FMatrix ColorWeights( FPlane(1, 0, 0, 0), FPlane(0, 1, 0, 0), FPlane(0, 0, 1, 0), FPlane(0, 0, 0, 0) );

	// this is the inverse of the gamma of the target; 
	//= 1.0 if "NoGamma" (EnableGammaCorrection(false))
	//= 1.0/2.2 to output with LinearToSRGB
	float GammaToUse = Gamma;

	ESimpleElementBlendMode MaskedBlendMode = SE_BLEND_Opaque;

	if(BlendMode >= SE_BLEND_RGBA_MASK_START && BlendMode <= SE_BLEND_RGBA_MASK_END)
	{
		/*
		* Red, Green, Blue and Alpha color weights all initialized to 0
		*/
		FPlane R( 0.0f, 0.0f, 0.0f, 0.0f );
		FPlane G( 0.0f, 0.0f, 0.0f, 0.0f );
		FPlane B( 0.0f, 0.0f, 0.0f, 0.0f );
		FPlane A( 0.0f,	0.0f, 0.0f, 0.0f );

		/*
		* Extract the color components from the in BlendMode to determine which channels should be active
		*/
		uint32 BlendMask = ( ( uint32 )BlendMode ) - SE_BLEND_RGBA_MASK_START;

		bool bRedChannel = ( BlendMask & ( 1 << 0 ) ) != 0;
		bool bGreenChannel = ( BlendMask & ( 1 << 1 ) ) != 0;
		bool bBlueChannel = ( BlendMask & ( 1 << 2 ) ) != 0;
		bool bAlphaChannel = ( BlendMask & ( 1 << 3 ) ) != 0;
		bool bDesaturate = ( BlendMask & ( 1 << 4 ) ) != 0;
		bool bAlphaOnly = bAlphaChannel && !bRedChannel && !bGreenChannel && !bBlueChannel;
		uint32 NumChannelsOn = ( bRedChannel ? 1 : 0 ) + ( bGreenChannel ? 1 : 0 ) + ( bBlueChannel ? 1 : 0 );
		GammaToUse = bAlphaOnly? 1.0f: Gamma;
		
		// If we are only to draw the alpha channel, make the Blend state opaque, to allow easy identification of the alpha values
		if( bAlphaOnly )
		{
			MaskedBlendMode = SE_BLEND_Opaque;
			SetBlendState(RHICmdList, GraphicsPSOInit, MaskedBlendMode);
			
			R.W = G.W = B.W = 1.0f;
		}
		else
		{
			MaskedBlendMode = !bAlphaChannel ? SE_BLEND_Opaque : SE_BLEND_Translucent;  // If alpha channel is disabled, do not allow alpha blending
			SetBlendState(RHICmdList, GraphicsPSOInit, MaskedBlendMode);

			// Determine the red, green, blue and alpha components of their respective weights to enable that colours prominence
			R.X = bRedChannel ? 1.0f : 0.0f;
			G.Y = bGreenChannel ? 1.0f : 0.0f;
			B.Z = bBlueChannel ? 1.0f : 0.0f;
			A.W = bAlphaChannel ? 1.0f : 0.0f;

			/*
			* Determine if desaturation is enabled, if so, we determine the output colour based on the number of active channels that are displayed
			* e.g. if Only the red and green channels are being viewed, then the desaturation of the image will be divided by 2
			*/
			if( bDesaturate && NumChannelsOn )
			{
				float ValR, ValG, ValB;
				ValR = R.X / NumChannelsOn;
				ValG = G.Y / NumChannelsOn;
				ValB = B.Z / NumChannelsOn;
				R = FPlane( ValR, ValG, ValB, 0 );
				G = FPlane( ValR, ValG, ValB, 0 );
				B = FPlane( ValR, ValG, ValB, 0 );
			}
		}

		ColorWeights = FMatrix( R, G, B, A );
	}
	
	if( BatchedElementParameters != NULL )
	{
		// Use the vertex/pixel shader that we were given
		FMatrix WorldToClip(ViewMatrices.RelativeWorldToClip);
		WorldToClip.SetOrigin(WorldToClip.GetOrigin() + FVector(ViewMatrices.PositionHigh));
		BatchedElementParameters->BindShaders(RHICmdList, GraphicsPSOInit, FeatureLevel, WorldToClip, GammaToUse, ColorWeights, Texture);
	}
	else
	{
		TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(FeatureLevel));
			
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

		if (bHitTesting)
		{
			SetHitTestingBlendState(RHICmdList, GraphicsPSOInit, BlendMode);

			TShaderMapRef<FSimpleElementHitProxyPS> HitTestingPixelShader(GetGlobalShaderMap(FeatureLevel));
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = HitTestingPixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

			SetShaderParametersLegacyPS(RHICmdList, HitTestingPixelShader, Texture);
		}
		else
		{
			if (BlendMode == SE_BLEND_Masked)
			{
				// use clip() in the shader instead of alpha testing as cards that don't support floating point blending
				// also don't support alpha testing to floating point render targets
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

				if (Texture->bSRGB)
				{
					auto MaskedPixelShader = GetPixelShader<FSimpleElementMaskedGammaPS_SRGB>(BlendMode, FeatureLevel);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = MaskedPixelShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);


					SetShaderParametersLegacyPS(RHICmdList, MaskedPixelShader, View, Texture, Gamma, OpacityMaskRefVal, BlendMode);
				}
				else
				{
					auto MaskedPixelShader = GetPixelShader<FSimpleElementMaskedGammaPS_Linear>(BlendMode, FeatureLevel);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = MaskedPixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

					SetShaderParametersLegacyPS(RHICmdList, MaskedPixelShader, View, Texture, Gamma, OpacityMaskRefVal, BlendMode);
				}
			}
			// render distance field elements
			else if (
				BlendMode == SE_BLEND_MaskedDistanceField || 
				BlendMode == SE_BLEND_MaskedDistanceFieldShadowed ||
				BlendMode == SE_BLEND_TranslucentDistanceField	||
				BlendMode == SE_BLEND_TranslucentDistanceFieldShadowed)
			{
				float AlphaRefVal = OpacityMaskRefVal;
				if (BlendMode == SE_BLEND_TranslucentDistanceField ||
					BlendMode == SE_BLEND_TranslucentDistanceFieldShadowed)
				{
					// enable alpha blending and disable clip ref value for translucent rendering
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();
					AlphaRefVal = 0.0f;
				}
				else
				{
					// clip is done in shader so just render opaque
					GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				}
				
				TShaderMapRef<FSimpleElementDistanceFieldGammaPS> DistanceFieldPixelShader(GetGlobalShaderMap(FeatureLevel));
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = DistanceFieldPixelShader.GetPixelShader();

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

				// @todo - expose these as options for batch rendering
				static FVector2D ShadowDirection(-1.0f/Texture->GetSizeX(),-1.0f/Texture->GetSizeY());
				static FLinearColor ShadowColor(FLinearColor::Black);
				const float ShadowSmoothWidth = (GBatchedElementSmoothWidth * 2) / Texture->GetSizeX();
				
				const bool EnableShadow = (
					BlendMode == SE_BLEND_MaskedDistanceFieldShadowed || 
					BlendMode == SE_BLEND_TranslucentDistanceFieldShadowed
					);

				SetShaderParametersLegacyPS(
					RHICmdList,
					DistanceFieldPixelShader,
					Texture,
					Gamma,
					AlphaRefVal,
					GBatchedElementSmoothWidth,
					EnableShadow,
					ShadowDirection,
					ShadowColor,
					ShadowSmoothWidth,
					(GlowInfo != NULL) ? *GlowInfo : FDepthFieldGlowInfo(),
					BlendMode
					);
			}
			else if(BlendMode == SE_BLEND_TranslucentAlphaOnly || BlendMode == SE_BLEND_TranslucentAlphaOnlyWriteAlpha)
			{
				SetBlendState(RHICmdList, GraphicsPSOInit, BlendMode);

				if (FMath::Abs(Gamma - 1.0f) < UE_KINDA_SMALL_NUMBER)
				{
					auto AlphaOnlyPixelShader = GetPixelShader<FSimpleElementAlphaOnlyPS>(BlendMode, FeatureLevel);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = AlphaOnlyPixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

					SetShaderParametersLegacyPS(RHICmdList, AlphaOnlyPixelShader, View, Texture);
				}
				else
				{
					auto GammaAlphaOnlyPixelShader = GetPixelShader<FSimpleElementGammaAlphaOnlyPS>(BlendMode, FeatureLevel);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = GammaAlphaOnlyPixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

					SetShaderParametersLegacyPS(RHICmdList, GammaAlphaOnlyPixelShader, View, Texture, Gamma, BlendMode);
				}
			}
			else if(BlendMode >= SE_BLEND_RGBA_MASK_START && BlendMode <= SE_BLEND_RGBA_MASK_END)
			{
				TShaderMapRef<FSimpleElementColorChannelMaskPS> ColorChannelMaskPixelShader(GetGlobalShaderMap(FeatureLevel));
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ColorChannelMaskPixelShader.GetPixelShader();

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
			
				SetShaderParametersLegacyPS(RHICmdList, ColorChannelMaskPixelShader, Texture, ColorWeights, GammaToUse);
			}
			else
			{
				SetBlendState(RHICmdList, GraphicsPSOInit, BlendMode);
	
				if (FMath::Abs(Gamma - 1.0f) < UE_KINDA_SMALL_NUMBER)
				{
					// runs "Main"
					TShaderMapRef<FSimpleElementPS> PixelShader(GetGlobalShaderMap(FeatureLevel));
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

					SetShaderParametersLegacyPS(RHICmdList, PixelShader, View, Texture);
				}
				else
				{
					// runs "GammaMain"
					TShaderRef<FSimpleElementGammaBasePS> BasePixelShader;

					// these shaders differ in setting SRGB_INPUT_TEXTURE, which is ignored
					//  so they are in fact the same
					if (Texture->bSRGB)
					{
						TShaderMapRef<FSimpleElementGammaPS_SRGB> PixelShader_SRGB(GetGlobalShaderMap(FeatureLevel));
						BasePixelShader = PixelShader_SRGB;
					}
					else
					{
						TShaderMapRef<FSimpleElementGammaPS_Linear> PixelShader_Linear(GetGlobalShaderMap(FeatureLevel));
						BasePixelShader = PixelShader_Linear;
					}

					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = BasePixelShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

					SetShaderParametersLegacyPS(RHICmdList, BasePixelShader, View, Texture, Gamma, BlendMode);
				}
			}
		}

		// Set the simple element vertex shader parameters
		const FSceneView* InstancedView = View->GetInstancedSceneView();
		SetShaderParametersLegacyVS(RHICmdList, VertexShader, ViewMatrices,
			InstancedView ? FDFRelativeViewMatrices::Create(InstancedView->ViewMatrices)
						  : ViewMatrices);
	}
}

void FBatchedElements::DrawPointElements(FRHICommandList& RHICmdList, const FMatrix& Transform, const uint32 ViewportSizeX, const uint32 ViewportSizeY, const FVector& CameraX, const FVector& CameraY, const uint32 NumInstances) const
{
	// Draw the point elements.
	if( Points.Num() > 0 )
	{
		// preallocate some memory to directly fill out
		const uint32 NumTris = ((uint32)Points.Num()) * 2; // even if Points.Num() == INT32_MAX, this won't overflow a uint32
		const uint32 NumVertices = NumTris * 3; // but this could

		// Prevent integer overflow to buffer overflow.
		if (NumTris > (UINT32_MAX / 3) ||
			NumVertices > UINT32_MAX / sizeof(FSimpleElementVertex))
		{
			UE_LOGF(LogBatchedElements, Error, "Too many points. Will overflow uint32 buffer size. NumPoints: %d", Points.Num());
			return;
		}

		const FRHIBufferCreateDesc Desc =
			FRHIBufferCreateDesc::CreateVertex<FSimpleElementVertex>(TEXT("FBatchedElements_Points"), NumVertices)
			.AddUsage(EBufferUsageFlags::Volatile)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FSimpleElementVertex> Initializer = RHICmdList.CreateBufferInitializer(Desc);

		uint32 VertIdx = 0;
		for(int32 PointIndex = 0; PointIndex < Points.Num(); PointIndex++)
		{
			// TODO: Support quad primitives here
			const FBatchedPoint& Point = Points[PointIndex];
			FVector4 TransformedPosition = Transform.TransformFVector4(Point.Position);

			// Generate vertices for the point such that the post-transform point size is constant.
			const uint32 ViewportMajorAxis = ViewportSizeX;//FMath::Max(ViewportSizeX, ViewportSizeY);
			const FVector WorldPointX = CameraX * Point.Size / ViewportMajorAxis * TransformedPosition.W;
			const FVector WorldPointY = CameraY * -Point.Size / ViewportMajorAxis * TransformedPosition.W;

			Initializer[VertIdx + 0] = FSimpleElementVertex(Point.Position + WorldPointX - WorldPointY,FVector2D(1,0),Point.Color,Point.HitProxyColor);
			Initializer[VertIdx + 1] = FSimpleElementVertex(Point.Position + WorldPointX + WorldPointY,FVector2D(1,1),Point.Color,Point.HitProxyColor);
			Initializer[VertIdx + 2] = FSimpleElementVertex(Point.Position - WorldPointX - WorldPointY,FVector2D(0,0),Point.Color,Point.HitProxyColor);
			Initializer[VertIdx + 3] = FSimpleElementVertex(Point.Position + WorldPointX + WorldPointY,FVector2D(1,1),Point.Color,Point.HitProxyColor);
			Initializer[VertIdx + 4] = FSimpleElementVertex(Point.Position - WorldPointX - WorldPointY,FVector2D(0,0),Point.Color,Point.HitProxyColor);
			Initializer[VertIdx + 5] = FSimpleElementVertex(Point.Position - WorldPointX + WorldPointY,FVector2D(0,1),Point.Color,Point.HitProxyColor);

			VertIdx += 6;
		}

		FBufferRHIRef VertexBufferRHI = Initializer.Finalize();

		RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
		RHICmdList.DrawPrimitive(0, NumTris, NumInstances);
	}
}

FSceneView FBatchedElements::CreateProxySceneView(const FMatrix& ProjectionMatrix, const FIntRect& ViewRect)
{
	FSceneViewInitOptions ProxyViewInitOptions;
	ProxyViewInitOptions.SetViewRectangle(ViewRect);
	ProxyViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ProxyViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ProxyViewInitOptions.ProjectionMatrix = ProjectionMatrix;

	return FSceneView(ProxyViewInitOptions);
}

bool FBatchedElements::Draw(FRHICommandList& RHICmdList, const FMeshPassProcessorRenderState& DrawRenderState, ERHIFeatureLevel::Type FeatureLevel, const FSceneView& View, bool bHitTesting, float Gamma /* = 1.0f */, EBlendModeFilter::Type Filter /* = EBlendModeFilter::All */, uint32 NumInstances /* = 1 */) const
{
	const FDFRelativeViewMatrices RelativeMatrices = FDFRelativeViewMatrices::Create(View.ViewMatrices);
	const FMatrix& WorldToClip = View.ViewMatrices.GetWorldToClip();
	const FMatrix& ClipToWorld = View.ViewMatrices.GetClipToWorld();
	const uint32 ViewportSizeX = View.UnscaledViewRect.Width();
	const uint32 ViewportSizeY = View.UnscaledViewRect.Height();

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	DrawRenderState.ApplyToPSO(GraphicsPSOInit);
	uint32 StencilRef = DrawRenderState.GetStencilRef();

	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();

	if (UNLIKELY(!FApp::CanEverRender()))
	{
		return false;
	}

	if( HasPrimsToDraw() )
	{
		FVector CameraX = ClipToWorld.TransformVector(FVector(1,0,0)).GetSafeNormal();
		FVector CameraY = ClipToWorld.TransformVector(FVector(0,1,0)).GetSafeNormal();
		FVector CameraZ = ClipToWorld.TransformVector(FVector(0,0,1)).GetSafeNormal();

		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();

		if( (LineVertices.Num() > 0 || Points.Num() > 0 || ThickLines.Num() > 0 || WireTris.Num() > 0)
			&& (Filter & EBlendModeFilter::OpaqueAndMasked))
		{
			// Lines/points don't support batched element parameters (yet!)
			FBatchedElementParameters* BatchedElementParameters = NULL;

			// Draw the line elements.
			if( LineVertices.Num() > 0 )
			{
				// Prevent integer overflow to buffer overflow.
				if (LineVertices.Num() > UINT32_MAX / sizeof(FSimpleElementVertex))
				{
					UE_LOGF(LogBatchedElements, Error, "Too many line vertices. Will overflow uint32 buffer size. LineVertices: %d", LineVertices.Num());
					return false;
				}

				GraphicsPSOInit.PrimitiveType = PT_LineList;

				// Set the appropriate pixel shader parameters & shader state for the non-textured elements.
				PrepareShaders(RHICmdList, GraphicsPSOInit, StencilRef, FeatureLevel, SE_BLEND_Opaque, RelativeMatrices, BatchedElementParameters, GWhiteTexture, bHitTesting, Gamma, NULL, &View);

				FBufferRHIRef VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("Lines"), BUF_Volatile, MakeConstArrayView(LineVertices));

				RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);

				int32 MaxVerticesAllowed = ((GDrawUPVertexCheckCount / sizeof(FSimpleElementVertex)) / 2) * 2;
				/*
				hack to avoid a crash when trying to render large numbers of line segments.
				*/
				MaxVerticesAllowed = FMath::Min(MaxVerticesAllowed, 64 * 1024);

				int32 MinVertex=0;
				int32 TotalVerts = (LineVertices.Num() / 2) * 2;
				while( MinVertex < TotalVerts )
				{
					int32 NumLinePrims = FMath::Min( MaxVerticesAllowed, TotalVerts - MinVertex ) / 2;
					RHICmdList.DrawPrimitive(MinVertex, NumLinePrims, NumInstances);
					MinVertex += NumLinePrims * 2;
				}

				VertexBufferRHI.SafeRelease();
			}

			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			// Set the appropriate pixel shader parameters & shader state for the non-textured elements.
			PrepareShaders(RHICmdList, GraphicsPSOInit, StencilRef, FeatureLevel, SE_BLEND_Opaque, RelativeMatrices, BatchedElementParameters, GWhiteTexture, bHitTesting, Gamma, NULL, &View);

			// Draw points
			DrawPointElements(RHICmdList, WorldToClip, ViewportSizeX, ViewportSizeY, CameraX, CameraY, NumInstances);

			if ( ThickLines.Num() > 0 )
			{
				float OrthoZoomFactor = 1.0f;

				const bool bIsPerspective = View.ViewMatrices.GetViewToClip().M[3][3] < 1.0f ? true : false;
				if (!bIsPerspective)
				{
					OrthoZoomFactor = 1.0f / View.ViewMatrices.GetViewToClip().M[0][0];
				}

				const double CameraScreenXScale = FVector(WorldToClip.TransformVector(CameraX)).Length();
				const double CameraScreenYScale = FVector(WorldToClip.TransformVector(CameraY)).Length();

				int32 LineIndex = 0;
				int32 MaxVerticesAllowed = ((GDrawUPVertexCheckCount / sizeof(FSimpleElementVertex)) / 3) * 3;
				/*
				hack to avoid a crash when trying to render large numbers of line segments.
				*/
				MaxVerticesAllowed = FMath::Min(MaxVerticesAllowed, 64 * 1024);

				constexpr int32 TrisPerLine = 4;
				constexpr int32 VertPerLine = 3 * TrisPerLine;
				const int32 MaxLinesPerBatch = MaxVerticesAllowed / VertPerLine;
				while (LineIndex < ThickLines.Num())
				{
					int32 FirstLineThisBatch = LineIndex;
					float DepthBiasThisBatch = ThickLines[LineIndex].DepthBias;
					while (++LineIndex < ThickLines.Num())
					{
						if ((ThickLines[LineIndex].DepthBias != DepthBiasThisBatch)
							|| ((LineIndex - FirstLineThisBatch) >= MaxLinesPerBatch))
						{
							break;
						}
					}
					int32 NumLinesThisBatch = LineIndex - FirstLineThisBatch;
					check(NumLinesThisBatch > 0);

					const bool bEnableMSAA = true;
					FRasterizerStateInitializerRHI Initializer(FM_Solid, CM_None, 0, DepthBiasThisBatch, ERasterizerDepthClipMode::DepthClip, bEnableMSAA);
					auto RasterState = RHICreateRasterizerState(Initializer);
					GraphicsPSOInit.RasterizerState = RasterState.GetReference();
					PrepareShaders(RHICmdList, GraphicsPSOInit, StencilRef, FeatureLevel, SE_BLEND_AlphaBlend, RelativeMatrices, BatchedElementParameters, GWhiteTexture, bHitTesting, Gamma, NULL, &View);

					const FRHIBufferCreateDesc Desc =
						FRHIBufferCreateDesc::CreateVertex<FSimpleElementVertex>(TEXT("ThickLines"), VertPerLine * NumLinesThisBatch)
						.AddUsage(EBufferUsageFlags::Volatile)
						.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
						.SetInitActionInitializer();

					TRHIBufferInitializer<FSimpleElementVertex> InitialData = RHICmdList.CreateBufferInitializer(Desc);
					FSimpleElementVertex* ThickVertices = InitialData.GetWritableData();

					int32 AddedTris = 0;
					for (int i = 0; i < NumLinesThisBatch; ++i)
					{
						const FBatchedThickLines& Line = ThickLines[FirstLineThisBatch + i];
						const double Thickness = FMath::Abs( Line.Thickness );
						
						FVector4 StartClip	= WorldToClip.TransformFVector4(Line.Start);
						FVector4 EndClip	= WorldToClip.TransformFVector4(Line.End);

						// Manually clip thick lines start/end if they would go behind the near plane
						FVector LineStart = Line.Start;
						FVector LineEnd = Line.End;
						const double ClipAt = View.NearClippingDistance + UE_DOUBLE_KINDA_SMALL_NUMBER;
						if (bIsPerspective)
						{
							if (StartClip.W < ClipAt && EndClip.W < ClipAt)
							{
								continue;
							}
							else if (StartClip.W < ClipAt)
							{
								double Along = (ClipAt - StartClip.W) / (EndClip.W - StartClip.W);
								LineStart = FMath::Lerp(LineStart, LineEnd, Along);
								StartClip = FMath::Lerp(StartClip, EndClip, Along);
							}
							else if (EndClip.W < ClipAt)
							{
								double Along = (ClipAt - EndClip.W) / (StartClip.W - EndClip.W);
								LineEnd = FMath::Lerp(LineEnd, LineStart, Along);
								EndClip = FMath::Lerp(EndClip, StartClip, Along);
							}
						}

						const double StartW = StartClip.W;
						const double EndW = EndClip.W;

						// bScreenSpace controls if thickness is calculated in screen space or world space.
						const double ScalingStart	= Line.bScreenSpace ? StartW / ViewportSizeX : 1.0;
						const double ScalingEnd		= Line.bScreenSpace ? EndW   / ViewportSizeX : 1.0;

						const double CurrentOrthoZoomFactor = Line.bScreenSpace ? OrthoZoomFactor : 1.0;

						const double ScreenSpaceScaling = Line.bScreenSpace ? 2.0 : 1.0;

						const double StartThickness	= Thickness * ScreenSpaceScaling * CurrentOrthoZoomFactor * ScalingStart;
						const double EndThickness	= Thickness * ScreenSpaceScaling * CurrentOrthoZoomFactor * ScalingEnd;

						// Line start and end points S and E are expanded into quads based on the line thickness, and
						// the line is filled in by trapezoids S2,S0,E0,E1 and S2,S3,E3,E1, w/ vertices as pictured:
						//      E3---E1
						//     / |...|
						//    /  |...|
						//   /  E2---E0
						// S3---S1  /
						//  |...|  /
						//  |...| /
						// S2---S0
						// This labelling assume the direction from S->E on screen is up-right (pictured)
						// We mirror point labels as needed to handle the down- or left- oriented cases

						// Figure out whether we're in an up-right/down-left case or not
						const double InvStartW = 1.0 / StartW, InvEndW = 1.0 / EndW;
						const double ScreenShiftX = EndClip.X * InvEndW - StartClip.X * InvStartW;
						const double ScreenShiftY = EndClip.Y * InvEndW - StartClip.Y * InvStartW;
						const bool bLineRight = ScreenShiftX >= 0;
						const bool bLineUp = ScreenShiftY >= 0;

						// Mirror X offsets in the up-left or down-right case
						const double XSign = bLineRight ? 1.0 : -1.0;
						const double YSign = bLineUp ? 1.0 : -1.0;
						// Mirror X for UVs as well
						const double RightUVX = (double)bLineRight, LeftUVX = (double)!bLineRight;
						const double UpUVY = (double)bLineUp, DownUVY = (double)!bLineUp;

						// Create the X and Y world offsets for each point, mirrored as needed
						const FVector WorldPointXS	= CameraX * (StartThickness * 0.5 * XSign);
						const FVector WorldPointYS	= CameraY * (StartThickness * 0.5 * YSign);
						const FVector WorldPointXE	= CameraX * (EndThickness * 0.5 * XSign);
						const FVector WorldPointYE	= CameraY * (EndThickness * 0.5 * YSign);

						// Vertex positions
						FVector S0 = LineStart + WorldPointXS - WorldPointYS;
						FVector S1 = LineStart + WorldPointXS + WorldPointYS;
						FVector S2 = LineStart - WorldPointXS - WorldPointYS;
						FVector S3 = LineStart - WorldPointXS + WorldPointYS;

						FVector E0 = LineEnd + WorldPointXE - WorldPointYE;
						FVector E1 = LineEnd + WorldPointXE + WorldPointYE;
						FVector E2 = LineEnd - WorldPointXE - WorldPointYE;
						FVector E3 = LineEnd - WorldPointXE + WorldPointYE;

						// UVs per vertex
						FVector2D S0UV = FVector2D(RightUVX, DownUVY);
						FVector2D S1UV = FVector2D(RightUVX, UpUVY);
						FVector2D S2UV = FVector2D(LeftUVX, DownUVY);
						FVector2D S3UV = FVector2D(LeftUVX, UpUVY);

						FVector2D E0UV = FVector2D(RightUVX, DownUVY);
						FVector2D E1UV = FVector2D(RightUVX, UpUVY);
						FVector2D E2UV = FVector2D(LeftUVX, DownUVY);
						FVector2D E3UV = FVector2D(LeftUVX, UpUVY);

						// Handle special cases due to one end of the line being larger on screen than the other -- these don't happen w/ screen space lines
						// Note: If you skip handling these cases, the lines will still mostly look fine, just not perfect if you get close to them
						if (!Line.bScreenSpace)
						{
							// Figure out if we're in a only-vertical (pictured) or only-horizontal case, like:
							//      E3---E1
							//     / |...| \
							//    /  |...|  \
							//   /  E2---E0  \
							// S3-------------S1
							//  |.............|
							// ... where S contains E along one axis, or vice versa
							// In these cases, replace one of the vertices with S1 or E2 to fix the missing corner
							const double ScreenStartRadius = StartThickness * InvStartW * .5;
							const double ScreenEndRadius = EndThickness * InvEndW * .5;
							const double RadDiff = ScreenEndRadius - ScreenStartRadius;
							const bool bStartIsBigger = RadDiff < 0;
							const double AbsRadDiff = bStartIsBigger ? -RadDiff : RadDiff;

							bool bOnlyVertical = ScreenShiftX * XSign < AbsRadDiff * CameraScreenXScale, bOnlyHorizontal = ScreenShiftY * YSign < AbsRadDiff * CameraScreenYScale;
							// If the larger point contains the smaller on both axes, we can just draw the larger point alone
							if (bOnlyVertical && bOnlyHorizontal)
							{
								if (bStartIsBigger)
								{
									// Tri: S0,S1,S2
									ThickVertices[0] = FSimpleElementVertex(S0, S0UV, Line.Color, Line.HitProxyColor);
									ThickVertices[1] = FSimpleElementVertex(S1, S1UV, Line.Color, Line.HitProxyColor);
									ThickVertices[2] = FSimpleElementVertex(S2, S2UV, Line.Color, Line.HitProxyColor);
									// Tri: S2,S1,S3
									ThickVertices[3] = FSimpleElementVertex(S2, S2UV, Line.Color, Line.HitProxyColor);
									ThickVertices[4] = FSimpleElementVertex(S1, S1UV, Line.Color, Line.HitProxyColor);
									ThickVertices[5] = FSimpleElementVertex(S3, S3UV, Line.Color, Line.HitProxyColor);
								}
								else
								{
									// Tri: E0,E1,E2
									ThickVertices[0] = FSimpleElementVertex(E0, E0UV, Line.Color, Line.HitProxyColor);
									ThickVertices[1] = FSimpleElementVertex(E1, E1UV, Line.Color, Line.HitProxyColor);
									ThickVertices[2] = FSimpleElementVertex(E2, E2UV, Line.Color, Line.HitProxyColor);
									// Tri: E2,E1,E3
									ThickVertices[3] = FSimpleElementVertex(E2, E2UV, Line.Color, Line.HitProxyColor);
									ThickVertices[4] = FSimpleElementVertex(E1, E1UV, Line.Color, Line.HitProxyColor);
									ThickVertices[5] = FSimpleElementVertex(E3, E3UV, Line.Color, Line.HitProxyColor);
								}
								ThickVertices += 6;
								AddedTris += 2;
								continue;
							}
							else if (bOnlyVertical) // only one direction is dominant; figure out which vertex to replace
							{
								if (bStartIsBigger)
								{
									// replace E0 with S1
									E0 = S1;
									E0UV = S1UV;
								}
								else
								{
									// replace S3 with E2
									S3 = E2;
									S3UV = E2UV;
								}
							}
							else if (bOnlyHorizontal)
							{
								if (bStartIsBigger)
								{
									// replace E3 with S1
									E3 = S1;
									E3UV = S1UV;
								}
								else
								{
									// replace S0 with E2
									S0 = E2;
									S0UV = E2UV;
								}
							}
						}
						

						// First trapezoid (S2,S0,E0,E1)
						// Tri: S2,S0,E0
						ThickVertices[0] = FSimpleElementVertex(S2, S2UV, Line.Color, Line.HitProxyColor);
						ThickVertices[1] = FSimpleElementVertex(S0, S0UV, Line.Color, Line.HitProxyColor);
						ThickVertices[2] = FSimpleElementVertex(E0, E0UV, Line.Color, Line.HitProxyColor);
						// Tri: S2,E1,E0
						ThickVertices[3] = FSimpleElementVertex(S2, S2UV, Line.Color, Line.HitProxyColor);
						ThickVertices[4] = FSimpleElementVertex(E1, E1UV, Line.Color, Line.HitProxyColor);
						ThickVertices[5] = FSimpleElementVertex(E0, E0UV, Line.Color, Line.HitProxyColor);

						// Second trapezoid (S2,S3,E3,E1)
						// Tri: S2,E3,E1
						ThickVertices[6] = FSimpleElementVertex(S2, S2UV, Line.Color, Line.HitProxyColor);
						ThickVertices[7] = FSimpleElementVertex(E3, E3UV, Line.Color, Line.HitProxyColor);
						ThickVertices[8] = FSimpleElementVertex(E1, E1UV, Line.Color, Line.HitProxyColor);
						// Tri: S2,S3,E3
						ThickVertices[9] = FSimpleElementVertex(S2, S2UV, Line.Color, Line.HitProxyColor);
						ThickVertices[10] = FSimpleElementVertex(S3, S3UV, Line.Color, Line.HitProxyColor);
						ThickVertices[11] = FSimpleElementVertex(E3, E3UV, Line.Color, Line.HitProxyColor);

						ThickVertices += VertPerLine;
						AddedTris += 4;
					}

					FBufferRHIRef VertexBufferRHI = InitialData.Finalize();

					if (AddedTris > 0)
					{
						RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
						RHICmdList.DrawPrimitive(0, AddedTris, NumInstances);
					}
				}

				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			}
			// Draw the wireframe triangles.
			if (WireTris.Num() > 0)
			{
				check(WireTriVerts.Num() == WireTris.Num() * 3);

				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex(TEXT("WireTris"), WireTriVerts.GetResourceDataSize())
					.AddUsage(BUF_Volatile)
					.SetInitActionResourceArray(&WireTriVerts)
					.SetInitialState(ERHIAccess::VertexOrIndexBuffer);

				FBufferRHIRef VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);

				RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);

				const bool bEnableMSAA = true;
				FRasterizerStateInitializerRHI Initializer(FM_Wireframe, CM_None, bEnableMSAA);

				int32 MaxVerticesAllowed = ((GDrawUPVertexCheckCount / sizeof(FSimpleElementVertex)) / 3) * 3;
				/*
				hack to avoid a crash when trying to render large numbers of line segments.
				*/
				MaxVerticesAllowed = FMath::Min(MaxVerticesAllowed, 64 * 1024);

				const int32 MaxTrisAllowed = MaxVerticesAllowed / 3;

				int32 MinTri=0;
				int32 TotalTris = WireTris.Num();
				while( MinTri < TotalTris )
				{
					int32 MaxTri = FMath::Min(MinTri + MaxTrisAllowed, TotalTris);
					float DepthBias = WireTris[MinTri].DepthBias;
					for (int32 i = MinTri + 1; i < MaxTri; ++i)
					{
						if (DepthBias != WireTris[i].DepthBias)
						{
							MaxTri = i;
							break;
						}
					}

					Initializer.DepthBias = DepthBias;
					auto RasterState = RHICreateRasterizerState(Initializer);
					GraphicsPSOInit.RasterizerState = RasterState.GetReference();
					PrepareShaders(RHICmdList, GraphicsPSOInit, StencilRef, FeatureLevel, SE_BLEND_Opaque, RelativeMatrices, BatchedElementParameters, GWhiteTexture, bHitTesting, Gamma, NULL, &View);

					int32 NumTris = MaxTri - MinTri;
					RHICmdList.DrawPrimitive(MinTri * 3, NumTris, NumInstances);
					MinTri = MaxTri;
				}
				VertexBufferRHI.SafeRelease();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			}
		}

		// Draw the sprites.
		if( Sprites.Num() > 0 )
		{
			// Sprites don't support batched element parameters (yet!)
			FBatchedElementParameters* BatchedElementParameters = NULL;

			//Sort sprites by blend mode and texture
			struct FCompareSprite
			{
				explicit FCompareSprite(EBlendModeFilter::Type InFilter) : ValidBlendFilter(InFilter) {}

				FORCEINLINE bool operator()( const FBatchedElements::FBatchedSprite& SpriteA, const FBatchedElements::FBatchedSprite& SpriteB ) const
				{
					if (SpriteA.BlendMode != SpriteB.BlendMode)
					{
						const bool bBlendAValid = (ValidBlendFilter & SpriteA.BlendMode) != 0;
						const bool bBlendBValid = (ValidBlendFilter & SpriteB.BlendMode) != 0;
						if (bBlendAValid != bBlendBValid) return bBlendAValid; // we want valid blend modes to sort in front of invalid blend modes
						return SpriteA.BlendMode < SpriteB.BlendMode;
					}
					return SpriteA.Texture < SpriteB.Texture;
				}

				EBlendModeFilter::Type ValidBlendFilter;
			};

			TArray<FBatchedSprite> SortedSprites = Sprites;

			SortedSprites.Sort(FCompareSprite(Filter));

			// count the number of sprites that have valid blend modes
			// (they have been sorted to the front of the list)
			int32 ValidSpriteCount = 0;
			while (ValidSpriteCount < SortedSprites.Num() &&
				(Filter & GetBlendModeFilter((ESimpleElementBlendMode)SortedSprites[ValidSpriteCount].BlendMode)))
			{
				++ValidSpriteCount;
			}

			if (ValidSpriteCount > 0)
			{
				// Prevent integer overflow to buffer overflow.
				if (ValidSpriteCount > UINT32_MAX / sizeof(FSimpleElementVertex) * 6)
				{
					UE_LOGF(LogBatchedElements, Error, "Too many sprites. Will overflow uint32 buffer size. ValidSpriteCount: %d", ValidSpriteCount);
					return false;
				}

				const FRHIBufferCreateDesc Desc =
					FRHIBufferCreateDesc::CreateVertex<FSimpleElementVertex>(TEXT("Sprites"), ValidSpriteCount * 6)
					.AddUsage(EBufferUsageFlags::Volatile)
					.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
					.SetInitActionInitializer();

				TRHIBufferInitializer<FSimpleElementVertex> Initializer = RHICmdList.CreateBufferInitializer(Desc);
				FSimpleElementVertex* SpriteList = Initializer.GetWritableData();

				for (int32 SpriteIndex = 0; SpriteIndex < ValidSpriteCount; SpriteIndex++)
				{
					const FBatchedSprite& Sprite = SortedSprites[SpriteIndex];
					FSimpleElementVertex* Vertex = &SpriteList[SpriteIndex * 6];

					// Compute the sprite vertices.
					const FVector WorldSpriteX = CameraX * Sprite.SizeX;
					const FVector WorldSpriteY = CameraY * -Sprite.SizeY * GProjectionSignY;

					const float UStart = Sprite.U / Sprite.Texture->GetSizeX();
					const float UEnd = (Sprite.U + Sprite.UL) / Sprite.Texture->GetSizeX();
					const float VStart = Sprite.V / Sprite.Texture->GetSizeY();
					const float VEnd = (Sprite.V + Sprite.VL) / Sprite.Texture->GetSizeY();

					Vertex[0] = FSimpleElementVertex(Sprite.Position + WorldSpriteX - WorldSpriteY, FVector2D(UEnd, VStart), Sprite.Color, Sprite.HitProxyColor);
					Vertex[1] = FSimpleElementVertex(Sprite.Position + WorldSpriteX + WorldSpriteY, FVector2D(UEnd, VEnd), Sprite.Color, Sprite.HitProxyColor);
					Vertex[2] = FSimpleElementVertex(Sprite.Position - WorldSpriteX - WorldSpriteY, FVector2D(UStart, VStart), Sprite.Color, Sprite.HitProxyColor);

					Vertex[3] = FSimpleElementVertex(Sprite.Position + WorldSpriteX + WorldSpriteY, FVector2D(UEnd, VEnd), Sprite.Color, Sprite.HitProxyColor);
					Vertex[4] = FSimpleElementVertex(Sprite.Position - WorldSpriteX - WorldSpriteY, FVector2D(UStart, VStart), Sprite.Color, Sprite.HitProxyColor);
					Vertex[5] = FSimpleElementVertex(Sprite.Position - WorldSpriteX + WorldSpriteY, FVector2D(UStart, VEnd), Sprite.Color, Sprite.HitProxyColor);
				}

				FBufferRHIRef VertexBufferRHI = Initializer.Finalize();

				//First time init
				const FTexture* CurrentTexture = SortedSprites[0].Texture;
				ESimpleElementBlendMode CurrentBlendMode = (ESimpleElementBlendMode)SortedSprites[0].BlendMode;
				int32 BatchStartIndex = 0;
				float CurrentOpacityMask = SortedSprites[0].OpacityMaskRefVal;

				// Start loop at 1, since we've already started the first batch with the first sprite in the list
				for (int32 SpriteIndex = 1; SpriteIndex < ValidSpriteCount + 1; SpriteIndex++)
				{
					// Need to flush the current batch once we hit the end of the list, or if state of this sprite doesn't match current batch
					if (SpriteIndex == ValidSpriteCount ||
						CurrentTexture != SortedSprites[SpriteIndex].Texture ||
						CurrentBlendMode != SortedSprites[SpriteIndex].BlendMode ||
						CurrentOpacityMask != SortedSprites[SpriteIndex].OpacityMaskRefVal)
					{
						const int32 SpriteNum = SpriteIndex - BatchStartIndex;
						const int32 BaseVertex = BatchStartIndex * 6;
						const int32 PrimCount = SpriteNum * 2;
						PrepareShaders(RHICmdList, GraphicsPSOInit, StencilRef, FeatureLevel, CurrentBlendMode, RelativeMatrices, BatchedElementParameters, CurrentTexture, bHitTesting, Gamma, NULL, &View, CurrentOpacityMask);
						RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
						RHICmdList.DrawPrimitive(BaseVertex, PrimCount, NumInstances);

						// begin the next batch
						if (SpriteIndex < ValidSpriteCount)
						{
							BatchStartIndex = SpriteIndex;
							CurrentTexture = SortedSprites[SpriteIndex].Texture;
							CurrentBlendMode = (ESimpleElementBlendMode)SortedSprites[SpriteIndex].BlendMode;
							CurrentOpacityMask = SortedSprites[SpriteIndex].OpacityMaskRefVal;
						}
					}
				}
				VertexBufferRHI.SafeRelease();
			}
		}

		if( MeshElements.Num() > 0)
		{
			// Prevent integer overflow to buffer overflow.
			if (MeshVertices.Num() > UINT32_MAX / sizeof(FSimpleElementVertex))
			{
				UE_LOGF(LogBatchedElements, Error, "Too many mesh vertices. Will overflow uint32 buffer size. MeshVertices.Num(): %d", MeshVertices.Num());
				return false;
			}

			FBufferRHIRef VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("MeshElements"), BUF_Volatile, MakeConstArrayView(MeshVertices));

			// Draw the mesh elements.
			for(int32 MeshIndex = 0;MeshIndex < MeshElements.Num();MeshIndex++)
			{
				const FBatchedMeshElement& MeshElement = MeshElements[MeshIndex];
				const EBlendModeFilter::Type MeshFilter = GetBlendModeFilter(MeshElement.BlendMode);

				// Only render blend modes in the filter
				if (Filter & MeshFilter)
				{
					// UE-50111
					checkf(
						MeshElement.Texture && MeshElement.Texture->SamplerStateRHI.GetReference(),
						TEXT("MeshElement.Texture=%p, bInitialized=%d, name=%s"),
						MeshElement.Texture,
						MeshElement.Texture ? MeshElement.Texture->IsInitialized() : -1,
						MeshElement.Texture ? *MeshElement.Texture->GetFriendlyName() : TEXT(""));

					// Set the appropriate pixel shader for the mesh.
					PrepareShaders(RHICmdList, GraphicsPSOInit, StencilRef, FeatureLevel, MeshElement.BlendMode, RelativeMatrices, MeshElement.BatchedElementParameters, MeshElement.Texture, bHitTesting, Gamma, &MeshElement.GlowInfo, &View);

					FBufferRHIRef IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("MeshElements"), EBufferUsageFlags::Volatile, MakeConstArrayView(MeshElement.Indices));

					// Draw the mesh.
					RHICmdList.SetStreamSource(0, VertexBufferRHI, MeshElement.MinVertex * sizeof(FSimpleElementVertex));
					RHICmdList.DrawIndexedPrimitive(IndexBufferRHI, 0, 0, MeshElement.MaxVertex - MeshElement.MinVertex + 1, 0, MeshElement.Indices.Num() / 3, NumInstances);

					IndexBufferRHI.SafeRelease();
				}
			}

			VertexBufferRHI.SafeRelease();
		}

		return true;
	}
	else
	{
		return false;
	}
}

void FBatchedElements::Clear()
{
	LineVertices.Empty();
	Points.Empty();
	WireTris.Empty();
	WireTriVerts.Empty();
	ThickLines.Empty();
	Sprites.Empty();
	MeshElements.Empty();
	MeshVertices.Empty();
}

void FBatchedElements::AddAllocationInfo(FAllocationInfo& AllocationInfo) const
{
	AllocationInfo.NumLineVertices += LineVertices.Num();
	AllocationInfo.NumPoints += Points.Num();
	AllocationInfo.NumWireTris += WireTris.Num();
	AllocationInfo.NumWireTriVerts += WireTriVerts.Num();
	AllocationInfo.NumThickLines += ThickLines.Num();
	AllocationInfo.NumSprites += Sprites.Num();
	AllocationInfo.NumMeshElements += MeshElements.Num();
	AllocationInfo.NumMeshVertices += MeshVertices.Num();
}

void FBatchedElements::Reserve(const FAllocationInfo& AllocationInfo)
{
	LineVertices.Reserve(AllocationInfo.NumLineVertices);
	Points.Reserve(AllocationInfo.NumPoints);
	WireTris.Reserve(AllocationInfo.NumWireTris);
	WireTriVerts.Reserve(AllocationInfo.NumWireTriVerts);
	ThickLines.Reserve(AllocationInfo.NumThickLines);
	Sprites.Reserve(AllocationInfo.NumSprites);
	MeshElements.Reserve(AllocationInfo.NumMeshElements);
	MeshVertices.Reserve(AllocationInfo.NumMeshVertices);
}

void FBatchedElements::Append(FBatchedElements& Other)
{
	LineVertices.Append(Other.LineVertices);
	Points.Append(Other.Points);
	WireTris.Append(Other.WireTris);
	WireTriVerts.Append(Other.WireTriVerts);
	ThickLines.Append(Other.ThickLines);
	Sprites.Append(Other.Sprites);
	MeshElements.Append(Other.MeshElements);
	MeshVertices.Append(Other.MeshVertices);
	Other.Clear();
}
