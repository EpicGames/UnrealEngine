// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/BlockingVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/BrushComponent.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlockingVolume)

static FName InvisibleWall_NAME("InvisibleWall");

ABlockingVolume::ABlockingVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCanEverAffectNavigation(true);
	GetBrushComponent()->SetCollisionProfileName(InvisibleWall_NAME);
}

#if WITH_EDITOR

static FName InvisibleWallDynamic_NAME("InvisibleWallDynamic");

void ABlockingVolume::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if(GetLinkerUEVersion() < VER_UE4_REMOVE_DYNAMIC_VOLUME_CLASSES)
	{
		static FName DynamicBlockingVolume_NAME("DynamicBlockingVolume");

		if(OldClassName == DynamicBlockingVolume_NAME)
		{
			GetBrushComponent()->Mobility = EComponentMobility::Movable;

			if(GetBrushComponent()->GetCollisionProfileName() == InvisibleWall_NAME)
			{
				GetBrushComponent()->SetCollisionProfileName(InvisibleWallDynamic_NAME);
			}
		}
	}
}

void ABlockingVolume::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	// Get 'deepest' property name we changed.
	const FName TailPropName = PropertyChangedEvent.PropertyChain.GetTail()->GetValue()->GetFName();
	static FName Mobility_NAME("Mobility");
	if( TailPropName == Mobility_NAME )
	{
		// If the collision profile is one of the 'default' ones for a BlockingVolume, make sure it is the correct one
		// If user has changed it to something else, don't touch it
		FName CurrentProfileName = GetBrushComponent()->GetCollisionProfileName();
		if(	CurrentProfileName == InvisibleWall_NAME ||
			CurrentProfileName == InvisibleWallDynamic_NAME )
		{
			if(GetBrushComponent()->Mobility == EComponentMobility::Movable)
			{
				GetBrushComponent()->SetCollisionProfileName(InvisibleWallDynamic_NAME);
			}
			else
			{
				GetBrushComponent()->SetCollisionProfileName(InvisibleWall_NAME);
			}
		}
	}
}

#endif

