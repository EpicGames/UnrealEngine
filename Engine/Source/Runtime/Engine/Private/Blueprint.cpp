// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Blueprint.h"

#include "Blueprint/BlueprintSupport.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "EngineLogs.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/PrimaryAssetId.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "Components/TimelineComponent.h"
#include "Modules/ModuleManager.h"
#include "UObject/TextProperty.h"

#if WITH_EDITOR
#include "BlueprintCompilationManager.h"
#include "Blueprints/BlueprintDependencies.h"
#include "Engine/SCS_Node.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/StructureEditorUtils.h"
#include "FindInBlueprintManager.h"
#include "Cooker/CookEvents.h"
#include "CookerSettings.h"
#include "Editor.h"
#include "Logging/MessageLog.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Engine/TimelineTemplate.h"
#include "Curves/CurveBase.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/MetaData.h"
#include "Blueprint/BlueprintExtension.h"
#include "UObject/TextProperty.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#endif

#include "Engine/InheritableComponentHandler.h"

#if WITH_EDITORONLY_DATA
#include "HAL/IConsoleManager.h"
#include "Kismet2/KismetDebugUtilities.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(Blueprint)

#if WITH_EDITOR
namespace UE::Private
{
	extern KISMET_API bool bRunArchetypePostLoadEarly;
}
#endif

DEFINE_LOG_CATEGORY(LogBlueprint);

//////////////////////////////////////////////////////////////////////////
// Static Helpers

#if WITH_EDITOR
/**
 * Updates the blueprint's OwnedComponents, such that they reflect changes made 
 * natively since the blueprint was last saved (a change in AttachParents, etc.)
 * 
 * @param  Blueprint	The blueprint whose components you wish to vet.
 */
void UBlueprint::ConformNativeComponents()
{
	if (UClass* const BlueprintClass = GeneratedClass)
	{
		if (AActor* BlueprintCDO = Cast<AActor>(BlueprintClass->GetDefaultObject(false)))
		{
			TInlineComponentArray<UActorComponent*> OldNativeComponents;
			// collect the native components that this blueprint was serialized out 
			// with (the native components it had last time it was saved)
			BlueprintCDO->GetComponents(OldNativeComponents);

			UClass* const NativeSuperClass = FBlueprintEditorUtils::FindFirstNativeClass(BlueprintClass);
			AActor* NativeCDO = CastChecked<AActor>(NativeSuperClass->GetDefaultObject(false));
			// collect the more up to date native components (directly from the 
			// native super-class)
			TInlineComponentArray<UActorComponent*> NewNativeComponents;
			NativeCDO->GetComponents(NewNativeComponents);
														   			
			// loop through all components that this blueprint thinks come from its
			// native super-class (last time it was saved)
			for (UActorComponent* Component : OldNativeComponents)
			{
				if (UActorComponent* NativeComponent = FComponentEditorUtils::FindMatchingComponent(Component, NewNativeComponents))
				{
					USceneComponent* SceneComponent = Cast<USceneComponent>(Component);
					if (SceneComponent == nullptr)
					{
						// if this isn't a scene-component, then we don't care
						// (we're looking to fixup scene-component parents)
						continue;
					}
					USceneComponent* OldNativeParent = Cast<USceneComponent>(FComponentEditorUtils::FindMatchingComponent(SceneComponent->GetAttachParent(), NewNativeComponents));
					USceneComponent* NativeSceneComponent = CastChecked<USceneComponent>(NativeComponent);
					// if this native component has since been reparented, we need
					// to make sure that this blueprint reflects that change
					if (OldNativeParent != NativeSceneComponent->GetAttachParent())
					{
						USceneComponent* NewParent = nullptr;
						if (NativeSceneComponent->GetAttachParent() != nullptr)
						{
							NewParent = CastChecked<USceneComponent>(FComponentEditorUtils::FindMatchingComponent(NativeSceneComponent->GetAttachParent(), OldNativeComponents));
						}
						SceneComponent->SetupAttachment(NewParent, SceneComponent->GetAttachSocketName());
					}
				}
				else
				{
					// the component has been removed from the native class
						// @TODO: I think we already handle removed native components elsewhere, so maybe we should error here?
	// 				BlueprintCDO->RemoveOwnedComponent(Component);
	// 
	// 				USimpleConstructionScript* BlueprintSCS = Blueprint->SimpleConstructionScript;
	// 				USCS_Node* ComponentNode = BlueprintSCS->CreateNode(Component, Component->GetFName());
	// 
	// 				BlueprintSCS->AddNode(ComponentNode);
				}
			}
		}
	}
}

#endif // WITH_EDITOR


//////////////////////////////////////////////////////////////////////////
// FBPVariableDescription

FBPVariableDescription::FBPVariableDescription()
	: PropertyFlags(CPF_Edit)
	, ReplicationCondition(ELifetimeCondition::COND_None)
{
}

int32 FBPVariableDescription::FindMetaDataEntryIndexForKey(const FName Key) const
{
	for(int32 i=0; i<MetaDataArray.Num(); i++)
	{
		if(MetaDataArray[i].DataKey == Key)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

bool FBPVariableDescription::HasMetaData(const FName Key) const
{
	return FindMetaDataEntryIndexForKey(Key) != INDEX_NONE;
}

/** Gets a metadata value on the variable; asserts if the value isn't present.  Check for validiy using FindMetaDataEntryIndexForKey. */
const FString& FBPVariableDescription::GetMetaData(const FName Key) const
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	check(EntryIndex != INDEX_NONE);
	return MetaDataArray[EntryIndex].DataValue;
}

void FBPVariableDescription::SetMetaData(const FName Key, FString Value)
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	if(EntryIndex != INDEX_NONE)
	{
		MetaDataArray[EntryIndex].DataValue = MoveTemp(Value);
	}
	else
	{
		MetaDataArray.Emplace( FBPVariableMetaDataEntry(Key, MoveTemp(Value)) );
	}
}

void FBPVariableDescription::RemoveMetaData(const FName Key)
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	if(EntryIndex != INDEX_NONE)
	{
		MetaDataArray.RemoveAtSwap(EntryIndex);
	}
}

//////////////////////////////////////////////////////////////////////////
// UBlueprintCore

#if WITH_EDITORONLY_DATA
namespace BlueprintLocalization
{
	static bool bSkipGraphProperties = true;
	static FAutoConsoleVariableRef CVarSkipGraphProperties(TEXT("Localization.SkipBlueprintGraphProperties"), bSkipGraphProperties, TEXT("True to skip BP graph properties from the localization gather, or false to mark them as editor-only"));

	void GatherForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
	{
		const UBlueprintCore* const BlueprintCore = CastChecked<UBlueprintCore>(Object);

		// Blueprint assets never exist at runtime, so treat all of their properties as editor-only, but allow their script (which is available at runtime) to be gathered by a game
		EPropertyLocalizationGathererTextFlags BlueprintGatherFlags = GatherTextFlags | (bSkipGraphProperties ? EPropertyLocalizationGathererTextFlags::SkipProperties : EPropertyLocalizationGathererTextFlags::ForceEditorOnlyProperties);

#if WITH_EDITOR
		if (const UBlueprint* const Blueprint = Cast<UBlueprint>(Object))
		{
			// Force non-data-only blueprints to set the HasScript flag, as they may not currently have bytecode due to a compilation error
			bool bForceHasScript = !FBlueprintEditorUtils::IsDataOnlyBlueprint(Blueprint);
			if (!bForceHasScript)
			{
				// Also do this for blueprints that derive from something containing text properties, as these may propagate default values from their parent class on load
				if (UClass* BlueprintParentClass = Blueprint->ParentClass.Get())
				{
					TArray<UStruct*> TypesToCheck;
					TypesToCheck.Add(BlueprintParentClass);

					TSet<UStruct*> TypesChecked;
					while (!bForceHasScript && TypesToCheck.Num() > 0)
					{
						UStruct* TypeToCheck = TypesToCheck.Pop(EAllowShrinking::No);
						TypesChecked.Add(TypeToCheck);

						for (TFieldIterator<const FProperty> PropIt(TypeToCheck, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); !bForceHasScript && PropIt; ++PropIt)
						{
							auto ProcessInnerProperty = [&bForceHasScript, &TypesToCheck, &TypesChecked](const FProperty* InProp) -> bool
							{
								if (const FTextProperty* TextProp = CastField<const FTextProperty>(InProp))
								{
									bForceHasScript = true;
									return true;
								}
								if (const FStructProperty* StructProp = CastField<const FStructProperty>(InProp))
								{
									if (!TypesChecked.Contains(StructProp->Struct))
									{
										TypesToCheck.Add(StructProp->Struct);
									}
									return true;
								}
								return false;
							};

							if (!ProcessInnerProperty(*PropIt))
							{
								if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(*PropIt))
								{
									ProcessInnerProperty(ArrayProp->Inner);
								}
								if (const FMapProperty* MapProp = CastField<const FMapProperty>(*PropIt))
								{
									ProcessInnerProperty(MapProp->KeyProp);
									ProcessInnerProperty(MapProp->ValueProp);
								}
								if (const FSetProperty* SetProp = CastField<const FSetProperty>(*PropIt))
								{
									ProcessInnerProperty(SetProp->ElementProp);
								}
							}
						}
					}
				}
			}

			if (bForceHasScript)
			{
				BlueprintGatherFlags |= EPropertyLocalizationGathererTextFlags::ForceHasScript;
			}
		}
#endif

		PropertyLocalizationDataGatherer.GatherLocalizationDataFromObject(BlueprintCore, BlueprintGatherFlags);
	}
}
#endif

UBlueprintCore::UBlueprintCore(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(UBlueprintCore::StaticClass(), &BlueprintLocalization::GatherForLocalization); }
#endif

	bLegacyNeedToPurgeSkelRefs = true;
}

void UBlueprintCore::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	Ar.UsingCustomVersion(FBlueprintsObjectVersion::GUID);
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::BlueprintGeneratedClassIsAlwaysAuthoritative)
	{
		// No longer in use.
		bool bLegacyGeneratedClassIsAuthoritative;
		Ar << bLegacyGeneratedClassIsAuthoritative;
	}
#endif

	if ((Ar.UEVer() < VER_UE4_BLUEPRINT_SKEL_CLASS_TRANSIENT_AGAIN)
		&& (Ar.UEVer() != VER_UE4_BLUEPRINT_SKEL_TEMPORARY_TRANSIENT))
	{
		Ar << SkeletonGeneratedClass;
		if( SkeletonGeneratedClass )
		{
			// If we serialized in a skeleton class, make sure it and all its children are updated to be transient
			SkeletonGeneratedClass->SetFlags(RF_Transient);
			TArray<UObject*> SubObjs;
			GetObjectsWithOuter(SkeletonGeneratedClass, SubObjs, EGetObjectsFlags::IncludeNestedObjects);
			for(auto SubObjIt = SubObjs.CreateIterator(); SubObjIt; ++SubObjIt)
			{
				(*SubObjIt)->SetFlags(RF_Transient);
			}
		}

		// We only want to serialize in the GeneratedClass if the SkeletonClass didn't trigger a recompile
		bool bSerializeGeneratedClass = true;
		if (UBlueprint* BP = Cast<UBlueprint>(this))
		{
			bSerializeGeneratedClass = !Ar.IsLoading() || !BP->bHasBeenRegenerated;
		}

		if (bSerializeGeneratedClass)
		{
			Ar << GeneratedClass;
		}
		else if (Ar.IsLoading())
		{
			UClass* DummyClass = NULL;
			Ar << DummyClass;
		}
	}

	if( Ar.IsLoading() && !BlueprintGuid.IsValid() )
	{
		GenerateDeterministicGuid();
	}
}

void UBlueprintCore::GenerateDeterministicGuid()
{
	FString HashString = GetPathName();
	HashString.Shrink();
	ensure( HashString.Len() );

	uint32 HashBuffer[ 5 ];
	uint32 BufferLength = HashString.Len() * sizeof( HashString[0] );
	FSHA1::HashBuffer(*HashString, BufferLength, reinterpret_cast< uint8* >( HashBuffer ));
	BlueprintGuid.A = HashBuffer[ 1 ];
	BlueprintGuid.B = HashBuffer[ 2 ];
	BlueprintGuid.C = HashBuffer[ 3 ];
	BlueprintGuid.D = HashBuffer[ 4 ];
}

UBlueprint::UBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, bRunConstructionScriptOnDrag(true)
	, bRunConstructionScriptInSequencer(false)
	, bGenerateConstClass(false)
#endif
#if WITH_EDITORONLY_DATA
	, bDuplicatingReadOnly(false)
	, bCachedDependenciesUpToDate(false)
#endif
{
}

#if WITH_EDITORONLY_DATA
static TAutoConsoleVariable<bool> CVarBPDisableSearchDataUpdateOnSave(
	TEXT("bp.DisableSearchDataUpdateOnSave"),
	false,
	TEXT("Don't update Blueprint search metadata on save (for QA/testing purposes only). On an editor relaunch, it should include the BP in the unindexed count after the first search."),
	ECVF_Cheat);

static TAutoConsoleVariable<bool> CVarBPForceOldSearchDataFormatVersionOnSave(
	TEXT("bp.ForceOldSearchDataFormatVersionOnSave"),
	false,
	TEXT("Force Blueprint search metadata to use an old format version on save (for QA/testing purposes only). On an editor relaunch, it should include the BP in the out-of-date count after the first search."),
	ECVF_Cheat);

void UBlueprint::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Clear all upgrade notes, the user has saved and should not see them anymore
	UpgradeNotesLog.Reset();
	const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();

	if (!TargetPlatform || TargetPlatform->HasEditorOnlyData())
	{
		// This will force an immediate (synchronous) update of this Blueprint's index tag value.
		EAddOrUpdateBlueprintSearchMetadataFlags Flags = EAddOrUpdateBlueprintSearchMetadataFlags::ForceRecache;

		// For regression testing, we exclude the registry tag on save by clearing the cached value.
		// Expected result: On an editor relaunch it should cause this BP to be reported as "unindexed," until the asset is loaded.
		if (CVarBPDisableSearchDataUpdateOnSave.GetValueOnGameThread())
		{
			Flags |= EAddOrUpdateBlueprintSearchMetadataFlags::ClearCachedValue;
		}

		// For regression testing, we allow an old format version to be used as an override on save.
		// Expected result: On an editor relaunch it should cause this BP to be reported as "out-of-date," until the asset is loaded.
		EFiBVersion OverrideVersion = EFiBVersion::FIB_VER_NONE;
		if (CVarBPForceOldSearchDataFormatVersionOnSave.GetValueOnGameThread())
		{
			OverrideVersion = EFiBVersion::FIB_VER_BASE;
		}

		// Cache the BP for use (immediate, since we're about to save)
		FFindInBlueprintSearchManager::Get().AddOrUpdateBlueprintSearchMetadata(this, Flags, OverrideVersion);

		FBlueprintEditorUtils::RecompileBeforeSaveIfNeeded(this);
	}
}

void UBlueprint::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	for (UClass* ClassIt = ParentClass; (ClassIt != NULL) && !(ClassIt->HasAnyClassFlags(CLASS_Native)); ClassIt = ClassIt->GetSuperClass())
	{
		if (ClassIt->ClassGeneratedBy)
		{
			OutDeps.Add(ClassIt->ClassGeneratedBy);
		}
	}
}
#endif // WITH_EDITORONLY_DATA

void UBlueprint::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if(Ar.IsLoading() && Ar.UEVer() < VER_UE4_BLUEPRINT_VARS_NOT_READ_ONLY)
	{
		// Allow all blueprint defined vars to be read/write.  undoes previous convention of making exposed variables read-only
		for (int32 i = 0; i < NewVariables.Num(); ++i)
		{
			FBPVariableDescription& Variable = NewVariables[i];
			Variable.PropertyFlags &= ~CPF_BlueprintReadOnly;
		}
	}

	if (Ar.UEVer() < VER_UE4_K2NODE_REFERENCEGUIDS)
	{
		for (int32 Index = 0; Index < NewVariables.Num(); ++Index)
		{
			NewVariables[Index].VarGuid = FGuid::NewGuid();
		}
	}

	// Preload our parent blueprints
	if (Ar.IsLoading())
	{
		for (UClass* ClassIt = ParentClass; ClassIt && !ClassIt->HasAnyClassFlags(CLASS_Native); ClassIt = ClassIt->GetSuperClass())
		{
			// In some cases, a non-native parent class may not have an associated Blueprint asset - we consider that to be ok here since we're just preloading.
			if (ClassIt->ClassGeneratedBy)
			{
				ClassIt->ClassGeneratedBy->ConditionalPreload();
			}
		}
	}

	for (int32 i = 0; i < NewVariables.Num(); ++i)
	{
		FBPVariableDescription& Variable = NewVariables[i];

		// Actor variables can't have default values (because Blueprint templates are library elements that can 
		// bridge multiple levels and different levels might not have the actor that the default is referencing).
		if (Ar.UEVer() < VER_UE4_FIX_BLUEPRINT_VARIABLE_FLAGS)
		{
			const FEdGraphPinType& VarType = Variable.VarType;

			bool bDisableEditOnTemplate = false;
			if (VarType.PinSubCategoryObject.IsValid()) // ignore variables that don't have associated objects
			{
				const UClass* ClassObject = Cast<UClass>(VarType.PinSubCategoryObject.Get());
				// if the object type is an actor...
				if ((ClassObject != NULL) && ClassObject->IsChildOf(AActor::StaticClass()))
				{
					// hide the default value field
					bDisableEditOnTemplate = true;
				}
			}

			if (bDisableEditOnTemplate)
			{
				Variable.PropertyFlags |= CPF_DisableEditOnTemplate;
			}
			else
			{
				Variable.PropertyFlags &= ~CPF_DisableEditOnTemplate;
			}
		}

		if (Ar.IsLoading())
		{
			// Validate metadata keys/values on load only
			FBlueprintEditorUtils::FixupVariableDescription(this, Variable);
		}
	}

	// cooked BP macros don't work well in cooked editor mixing with uncooked BPs, so we load pacakges with macros as uncooked
	// @todo: temporarily marking all BPs to be uncooked, must check in to get a build overnight
	if (!HasAnyFlags(RF_ClassDefaultObject) && Ar.IsPersistent())// && BlueprintType == EBlueprintType::BPTYPE_MacroLibrary)
	{
		GetOutermost()->ThisShouldLoadUncooked(Ar);
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR

bool UBlueprint::RenameGeneratedClasses(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	const bool bRenameGeneratedClasses = !(Flags & REN_SkipGeneratedClasses);

	if (bRenameGeneratedClasses)
	{
		const auto TryFreeName = [](const FString& NewClassString, UObject* NewOuter, ERenameFlags InFlags)
			{
				if (UObject* ExistingObject = StaticFindObjectFast(UObject::StaticClass(), NewOuter, *NewClassString))
				{
					const FName NewName = MakeUniqueObjectName(NewOuter, ExistingObject->GetClass(), *NewClassString);
					ExistingObject->Rename(*(NewName.ToString()), NewOuter, InFlags | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
				}
			};

		const auto CheckRedirectors = [](FName NewClassName, UClass* OldClass, UObject* NewOuter)
			{
				if (UObjectRedirector* Redirector = FindObjectFast<UObjectRedirector>(NewOuter, NewClassName))
				{
					// If we found a redirector, check that the object it points to is of the same class.
					const bool bFoundRedirector =
						Redirector->DestinationObject &&
						(Redirector->DestinationObject->GetClass() == OldClass->GetClass())
					;

					if (bFoundRedirector)
					{
						Redirector->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
					}
				}
			};

		FName NewSkelClassName;
		FName NewGenClassName;
		GetBlueprintClassNames(NewGenClassName, NewSkelClassName, FName(NewName));

		UPackage* NewTopLevelObjectOuter = NewOuter ? NewOuter->GetOutermost() : nullptr;
		if (GeneratedClass != nullptr)
		{
			const FString NewGenClassNameString = NewGenClassName.ToString();

			TryFreeName(NewGenClassNameString, NewTopLevelObjectOuter, Flags);
			CheckRedirectors(NewGenClassName, GeneratedClass, NewTopLevelObjectOuter);
			const bool bMovedOK = GeneratedClass->Rename(*NewGenClassNameString, NewTopLevelObjectOuter, Flags);
			if (!bMovedOK)
			{
				return false;
			}
		}

		// Also move skeleton class, if different from generated class, to new package (again, to create redirector)
		const bool bShouldMoveSkeleton =
			(SkeletonGeneratedClass != nullptr) && 
			(SkeletonGeneratedClass != GeneratedClass);

		if (bShouldMoveSkeleton)
		{
			const FString NewSkelClassNameString = NewSkelClassName.ToString();

			TryFreeName(NewSkelClassNameString, NewTopLevelObjectOuter, Flags);
			CheckRedirectors(NewSkelClassName, SkeletonGeneratedClass, NewTopLevelObjectOuter);
			const bool bMovedOK = SkeletonGeneratedClass->Rename(*NewSkelClassNameString, NewTopLevelObjectOuter, Flags);
			if (!bMovedOK)
			{
				return false;
			}
		}
	}
	return true;
}

bool UBlueprint::Rename( const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags )
{
	const FName OldName = GetFName();
	
	TArray<UObject*> LastEditedDocumentsObjects;
	if (!(Flags & REN_Test))
	{
		LastEditedDocumentsObjects.Reserve(LastEditedDocuments.Num());
		for (const FEditedDocumentInfo& LastEditedDocument : LastEditedDocuments)
		{
			LastEditedDocumentsObjects.Add(LastEditedDocument.EditedObjectPath.ResolveObject());
		}
	}

	// Move generated class/CDO to the new package, to create redirectors
	if ( !RenameGeneratedClasses(InName, NewOuter, Flags) )
	{
		return false;
	}

	if (Super::Rename( InName, NewOuter, Flags ))
	{
		if (!(Flags & REN_Test))
		{
			for (int32 i=0; i<LastEditedDocuments.Num(); i++)
			{
				if (LastEditedDocumentsObjects[i])
				{
					LastEditedDocuments[i].EditedObjectPath = FSoftObjectPath(LastEditedDocumentsObjects[i]);
				}
			}
		}
		return true;
	}

	return false;
}

void UBlueprint::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if( !bDuplicatingReadOnly )
	{
		FBlueprintEditorUtils::PostDuplicateBlueprint(this, bDuplicateForPIE);
	}

	if (GeneratedClass)
	{
		GeneratedClass->GetDefaultObject()->PostDuplicate(bDuplicateForPIE);
	}
}

UClass* UBlueprint::RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO)
{
	LoadModulesRequiredForCompilation();

	// ensure that we have UProperties for any properties declared in the blueprint:
	if(!GeneratedClass || !HasAnyFlags(RF_BeingRegenerated) || bIsRegeneratingOnLoad || bHasBeenRegenerated)
	{
		return GeneratedClass;
	}
		
	// tag ourself as bIsRegeneratingOnLoad so that any reentrance via ForceLoad calls doesn't recurse:
	bIsRegeneratingOnLoad = true;
		
	UPackage* Package = GetOutermost();
	bool bIsPackageDirty = Package ? Package->IsDirty() : false;

	UClass* GeneratedClassResolved = GeneratedClass;

	if (ensure(GeneratedClassResolved->GetDefaultObject(false)))
	{
		UBlueprint::ForceLoadMembers(GeneratedClassResolved);
		if(UE::Private::bRunArchetypePostLoadEarly)
		{
			UBlueprint::ForceLoadGraph(GeneratedClassResolved->GetDefaultObject(false));
		}
		else
		{
			UBlueprint::ForceLoadMembers(GeneratedClassResolved->GetDefaultObject(false));
		}
	}
	UBlueprint::ForceLoadMembers(this);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (auto It = Extensions.CreateIterator(); It; ++It)
	{
		if (!*It)
		{
			It.RemoveCurrent();
		}
	}
	
	for (UBlueprintExtension* Extension : Extensions)
	{
		ForceLoad(Extension);
		Extension->PreloadObjectsForCompilation(this);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FBlueprintEditorUtils::PreloadConstructionScript( this );

	FBlueprintEditorUtils::LinkExternalDependencies( this );

	FBlueprintEditorUtils::RefreshVariables(this);
		
	// Preload Overridden Components
	if (InheritableComponentHandler)
	{
		InheritableComponentHandler->PreloadAll();
	}

	FBlueprintCompilationManager::NotifyBlueprintLoaded( this ); 
		
	FBlueprintEditorUtils::PreloadBlueprintSpecificData( this );

	FBlueprintEditorUtils::UpdateOutOfDateAnimBlueprints(this);

	// clear this now that we're not in a re-entrrant context - bHasBeenRegenerated will guard against 'real' 
	// double regeneration calls:
	bIsRegeneratingOnLoad = false;

	if( Package )
	{
		Package->SetDirtyFlag(bIsPackageDirty);
	}

	return GeneratedClassResolved;
}

void UBlueprint::RemoveChildRedirectors()
{
	TArray<UObject*> ChildObjects;
	GetObjectsWithOuter(this, ChildObjects);
	for (UObject* ChildObject : ChildObjects)
	{
		if (ChildObject->IsA<UObjectRedirector>())
		{
			ChildObject->ClearFlags(RF_Public|RF_Standalone);
			ChildObject->SetFlags(RF_Transient);
			ChildObject->RemoveFromRoot();
		}
	}
}

void UBlueprint::RemoveGeneratedClasses()
{
	FBlueprintEditorUtils::RemoveGeneratedClasses(this);
}

void UBlueprint::PostLoad()
{
	Super::PostLoad();

	// Can't use TGuardValue here as bIsRegeneratingOnLoad is a bitfield
	struct FScopedRegeneratingOnLoad
	{
		UBlueprint& Blueprint;
		bool bPreviousValue;
		FScopedRegeneratingOnLoad(UBlueprint& InBlueprint)
			: Blueprint(InBlueprint)
			, bPreviousValue(InBlueprint.bIsRegeneratingOnLoad)
		{
			// if the blueprint's package is still in the midst of loading, then
			// bIsRegeneratingOnLoad needs to be set to prevent UObject renames
			// from resetting loaders
			Blueprint.bIsRegeneratingOnLoad = true;
			if (UPackage* Package = Blueprint.GetOutermost())
			{
				// checking (Package->LinkerLoad != nullptr) ensures this 
				// doesn't get set when duplicating blueprints (which also calls 
				// PostLoad), and checking RF_WasLoaded makes sure we only 
				// forcefully set bIsRegeneratingOnLoad for blueprints that need 
				// it (ones still actively loading)
				Blueprint.bIsRegeneratingOnLoad = bPreviousValue || ((Package->GetLinker() != nullptr) && !Package->HasAnyFlags(RF_WasLoaded));
			}
		}
		~FScopedRegeneratingOnLoad()
		{
			Blueprint.bIsRegeneratingOnLoad = bPreviousValue;
		}
	} GuardIsRegeneratingOnLoad(*this);

	// Mark the blueprint as in error if there has been a major version bump
	if (BlueprintSystemVersion < UBlueprint::GetCurrentBlueprintSystemVersion())
	{
		Status = BS_Error;
	}

	// Purge any NULL graphs
	FBlueprintEditorUtils::PurgeNullGraphs(this);

#if WITH_EDITOR
	// Restore breakpoints for this Blueprint
	FKismetDebugUtilities::RestoreBreakpointsOnLoad(this);
# endif
	Breakpoints_DEPRECATED.Empty();

	// Make sure we have an SCS and ensure it's transactional
	if( FBlueprintEditorUtils::SupportsConstructionScript(this) )
	{
		if(SimpleConstructionScript == nullptr)
		{
			check(nullptr != GeneratedClass);
			SimpleConstructionScript = NewObject<USimpleConstructionScript>(GeneratedClass);
			SimpleConstructionScript->SetFlags(RF_Transactional);

			UBlueprintGeneratedClass* BPGClass = Cast<UBlueprintGeneratedClass>(*GeneratedClass);
			if(BPGClass)
			{
				BPGClass->SimpleConstructionScript = SimpleConstructionScript;
			}
		}
		else
		{
			if (!SimpleConstructionScript->HasAnyFlags(RF_Transactional))
			{
				SimpleConstructionScript->SetFlags(RF_Transactional);
			}
		}
	}

	// Make sure the CDO's scene root component is valid
	FBlueprintEditorUtils::UpdateRootComponentReference(this);

	// Make sure all the components are used by this blueprint
	FBlueprintEditorUtils::UpdateComponentTemplates(this);

	// Make sure that all of the parent function calls are valid
	FBlueprintEditorUtils::ConformCallsToParentFunctions(this);

	// Make sure that all of the events this BP implements are valid
	FBlueprintEditorUtils::ConformImplementedEvents(this);

	// Make sure that all of the interfaces this BP implements have all required graphs
	FBlueprintEditorUtils::ConformImplementedInterfaces(this);

	// Make sure that there are no function graphs that are marked as bAllowDeletion=false 
	// (possible if a blueprint was reparented prior to 4.11).
	if (GetLinkerCustomVersion(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::AllowDeletionConformed)
	{
		FBlueprintEditorUtils::ConformAllowDeletionFlag(this);
	}

#if WITH_EDITORONLY_DATA
	// Ensure all the pin watches we have point to something useful
	FBlueprintEditorUtils::UpdateStalePinWatches(this);
#endif // WITH_EDITORONLY_DATA

	FStructureEditorUtils::RemoveInvalidStructureMemberVariableFromBlueprint(this);

#if WITH_EDITOR
	// Do not want to run this code without the editor present nor when running commandlets.
	if(GEditor && GIsEditor && !IsRunningCommandlet())
	{
		// Gathers Find-in-Blueprint data, makes sure that it is fresh and ready, especially if the asset did not have any available.
		FFindInBlueprintSearchManager::Get().AddOrUpdateBlueprintSearchMetadata(this);
	}
#endif
}

#if WITH_EDITORONLY_DATA
void UBlueprint::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UObjectRedirector::StaticClass()));
}
#endif


void UBlueprint::DebuggingWorldRegistrationHelper(UObject* ObjectProvidingWorld, UObject* ValueToRegister)
{
	if (ObjectProvidingWorld != NULL)
	{
		// Fix up the registration with the world
		UWorld* ObjWorld = NULL;
		UObject* ObjOuter = ObjectProvidingWorld->GetOuter();
		while (ObjOuter != NULL)
		{
			ObjWorld = Cast<UWorld>(ObjOuter);
			if (ObjWorld != NULL)
			{
				break;
			}

			ObjOuter = ObjOuter->GetOuter();
		}

		// if we can't find the world on the outer chain, fallback to the GetWorld method
		if (ObjWorld == NULL)
		{
			ObjWorld = ObjectProvidingWorld->GetWorld();
		}

		if (ObjWorld != NULL)
		{
			if( !ObjWorld->HasAnyFlags(RF_BeginDestroyed))
			{
				ObjWorld->NotifyOfBlueprintDebuggingAssociation(this, ValueToRegister);
			}
			OnSetObjectBeingDebuggedDelegate.Broadcast(ValueToRegister);
		}
	}
}

void UBlueprint::ForceLoadGraph(UObject* Object)
{
	// find every directly referenced object:
	TArray<UObject*> ReachableInnerObjects;
	ReachableInnerObjects.Add(Object);
	FReferenceFinder InnerCollector(ReachableInnerObjects, Object, false, true, true, true);
	InnerCollector.FindReferences(Object);

	// preload until we stabilize
	TSet<UObject*> LoadedObjects;
	TArray<UObject*> NewObjects = ReachableInnerObjects;
	while(NewObjects.Num())
	{
		LoadedObjects.Append(NewObjects);
		TArray<UObject*> ExternalObjects;
		for(UObject* InnerObject : NewObjects)
		{
			ForceLoad(InnerObject);
		}
		TArray<UObject*> ReachableObjects;
		for(UObject* InnerObject : NewObjects)
		{
			FReferenceFinder ComponentCollector(ReachableObjects, nullptr, false, true, true);
			ComponentCollector.FindReferences(InnerObject);
		}

		NewObjects.Reset();
		for(UObject* ReachableObject : ReachableObjects) //-V1078
		{
			if(LoadedObjects.Contains(ReachableObject))
			{
				continue;
			}

			if(ReachableObject->HasAnyFlags(RF_WasLoaded) && !ReachableObject->HasAnyFlags(RF_LoadCompleted))
			{
				NewObjects.Add(ReachableObject);
			}
			else
			{
				// make sure this objects loaded inner objects are also loaded - this specifically handles 
				// natively created objects like BodySetup that are not reachable by FReferenceFinder:
				TArray<UObject*> SubObjs;
				GetObjectsWithOuter(ReachableObject, SubObjs, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);
				for(UObject* SubObj : SubObjs)
				{
					if(SubObj->HasAnyFlags(RF_WasLoaded) && !SubObj->HasAnyFlags(RF_LoadCompleted))
					{
						NewObjects.Add(SubObj);
					}
				}
			}
		}
	}
}

#if WITH_EDITOR
void UBlueprint::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	using namespace UE::Cook;
	Super::OnCookEvent(CookEvent, CookContext);

	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		static bool bIgnoreBlueprintCookDependencies = FParse::Param(FCommandLine::Get(), TEXT("IgnoreBlueprintCookDependencies"));
		if (bIgnoreBlueprintCookDependencies)
		{
			return;
		}

		CookContext.AddLoadBuildDependency(
			UE::Private::BlueprintDependencies::RecordCookDependencies(this));
	}
}
#endif

UClass* UBlueprint::GetBlueprintClass() const
{
	return UBlueprintGeneratedClass::StaticClass();
}

EBlueprintCompileOptions UBlueprint::GetDefaultCompileOptions() const
{
	return EBlueprintCompileOptions::None;
}

void UBlueprint::SetObjectBeingDebugged(UObject* NewObject)
{
	// Unregister the old object (even if PendingKill)
	if (UObject* OldObject = CurrentObjectBeingDebugged.Get(true))
	{
		if (OldObject == NewObject)
		{
			// Nothing changed
			return;
		}

		DebuggingWorldRegistrationHelper(OldObject, nullptr);
	}

	// Note that we allow macro Blueprints to bypass this check
	if ((NewObject != nullptr) && !GCompilingBlueprint && BlueprintType != BPTYPE_MacroLibrary)
	{
		// You can only debug instances of this!
		if (!ensureMsgf(
				NewObject->IsA(this->GeneratedClass), 
				TEXT("Type mismatch: Expected %s, Found %s"), 
				this->GeneratedClass ? *(this->GeneratedClass->GetName()) : TEXT("NULL"), 
				NewObject->GetClass() ? *(NewObject->GetClass()->GetName()) : TEXT("NULL")))
		{
			NewObject = nullptr;
		}
	}

	// Update the current object being debugged
	CurrentObjectBeingDebugged = NewObject;

	// Register the new object
	if (NewObject != nullptr)
	{
		ObjectPathToDebug = NewObject->GetPathName();
		DebuggingWorldRegistrationHelper(NewObject, NewObject);
	}
	else
	{
		ObjectPathToDebug = FString();
	}
}

void UBlueprint::UnregisterObjectBeingDebugged()
{
	// This is implemented as a set to null and restore of ObjectPathToDebug, so subclasses have their overrides called properly
	FString LastPath = ObjectPathToDebug;
	SetObjectBeingDebugged(nullptr);
	ObjectPathToDebug = LastPath;
}

void UBlueprint::SetWorldBeingDebugged(UWorld *NewWorld)
{
	CurrentWorldBeingDebugged = NewWorld;
}

void UBlueprint::GetReparentingRules(TSet< const UClass* >& AllowedChildrenOfClasses, TSet< const UClass* >& DisallowedChildrenOfClasses) const
{

}

bool UBlueprint::CanAlwaysRecompileWhilePlayingInEditor() const
{
	return false;
}

void UBlueprint::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	// We use Generated instead of Skeleton because the CDO data is more accurate on Generated
	UObject* BlueprintCDO = nullptr;
	if (GeneratedClass)
	{
		BlueprintCDO = GeneratedClass->GetDefaultObject(/*bCreateIfNeeded*/false);
		if (BlueprintCDO)
		{
			BlueprintCDO->GetAssetRegistryTags(Context);
		}
	}

	Super::GetAssetRegistryTags(Context);

	// Output native parent class and generated class as if they were AssetRegistrySearchable
	FString GeneratedClassVal;
	if (GeneratedClass)
	{
		GeneratedClassVal = FObjectPropertyBase::GetExportPath(GeneratedClass);
	}
	else
	{
		GeneratedClassVal = TEXT("None");
	}

	FString NativeParentClassName, ParentClassName;
	if ( ParentClass )
	{
		ParentClassName = FObjectPropertyBase::GetExportPath(ParentClass);

		// Walk up until we find a native class (ie 'while they are BP classes')
		UClass* NativeParentClass = ParentClass;
		while (Cast<UBlueprintGeneratedClass>(NativeParentClass) != nullptr) // can't use IsA on UClass
		{
			NativeParentClass = NativeParentClass->GetSuperClass();
		}
		NativeParentClassName = FObjectPropertyBase::GetExportPath(NativeParentClass);
	}
	else
	{
		NativeParentClassName = ParentClassName = TEXT("None");
	}


	Context.AddTag(FAssetRegistryTag(FBlueprintTags::BlueprintPathWithinPackage, GetPathName(GetOutermost()), FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::GeneratedClassPath, GeneratedClassVal, FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::ParentClassPath, ParentClassName, FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::NativeParentClassPath, NativeParentClassName, FAssetRegistryTag::TT_Alphabetical));

	// BlueprintGeneratedClass is not automatically traversed so we have to manually add NumReplicatedProperties
	int32 NumReplicatedProperties = 0;
	UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(SkeletonGeneratedClass);
	if (BlueprintClass)
	{
		NumReplicatedProperties = BlueprintClass->NumReplicatedProperties;
	}
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::NumReplicatedProperties, FString::FromInt(NumReplicatedProperties), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::BlueprintDescription, BlueprintDescription, FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::BlueprintCategory, BlueprintCategory, FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::BlueprintDisplayName, BlueprintDisplayName, FAssetRegistryTag::TT_Hidden));

	uint32 ClassFlagsTagged = 0;
	if (BlueprintClass)
	{
		ClassFlagsTagged = BlueprintClass->GetClassFlags();
	}
	else
	{
		ClassFlagsTagged = GetClass()->GetClassFlags();
	}
	Context.AddTag( FAssetRegistryTag(FBlueprintTags::ClassFlags, FString::FromInt(ClassFlagsTagged), FAssetRegistryTag::TT_Hidden) );

	Context.AddTag( FAssetRegistryTag(FBlueprintTags::IsDataOnly,
			FBlueprintEditorUtils::IsDataOnlyBlueprint(this) ? TEXT("True") : TEXT("False"),
			FAssetRegistryTag::TT_Alphabetical ) );

	// Only add the FiB tags in the editor, this now gets run for standalone uncooked games
	if ( ParentClass && GIsEditor && !GetOutermost()->HasAnyPackageFlags(PKG_ForDiffing) && !IsRunningCookCommandlet())
	{
		FString Value;
		const bool bRebuildSearchData = false;
		FSearchData SearchData = FFindInBlueprintSearchManager::Get().QuerySingleBlueprint((UBlueprint*)this, bRebuildSearchData);
		if (SearchData.IsValid())
		{
			Value = SearchData.Value;
		}
		
		Context.AddTag( FAssetRegistryTag(FBlueprintTags::FindInBlueprintsData, Value, FAssetRegistryTag::TT_Hidden) );
	}

	// Only show for strict blueprints (not animation or widget blueprints)
	// Note: Can't be an Actor specific check on the gen class, as CB only queries the CDO for the majority type to determine which columns are shown in the view
	if (ExactCast<UBlueprint>(this) != nullptr)
	{
		// Determine how many inherited native components exist
		int32 NumNativeComponents = 0;
		if (BlueprintCDO != nullptr)
		{
			TArray<UObject*> PotentialComponents;
			BlueprintCDO->GetDefaultSubobjects(/*out*/ PotentialComponents);

			for (UObject* TestSubObject : PotentialComponents)
			{
				if (Cast<UActorComponent>(TestSubObject) != nullptr)
				{
					++NumNativeComponents;
				}
			}
		}
		Context.AddTag(FAssetRegistryTag(FBlueprintTags::NumNativeComponents, FString::FromInt(NumNativeComponents), UObject::FAssetRegistryTag::TT_Numerical));

		// Determine how many components are added via a SimpleConstructionScript (both newly introduced and inherited from parent BPs)
		int32 NumAddedComponents = 0;
		for (UBlueprintGeneratedClass* TestBPClass = BlueprintClass; TestBPClass != nullptr; TestBPClass = Cast<UBlueprintGeneratedClass>(TestBPClass->GetSuperClass()))
		{
			const UBlueprint* AssociatedBP = Cast<const UBlueprint>(TestBPClass->ClassGeneratedBy);
			if (AssociatedBP && AssociatedBP->SimpleConstructionScript != nullptr)
			{
				NumAddedComponents += AssociatedBP->SimpleConstructionScript->GetAllNodes().Num();
			}
		}
		Context.AddTag(FAssetRegistryTag(FBlueprintTags::NumBlueprintComponents, FString::FromInt(NumAddedComponents), UObject::FAssetRegistryTag::TT_Numerical));
	}

#if WITH_EDITOR
	if (Context.IsSaving())
	{
		if (ParentClass && GIsEditor && !GetOutermost()->HasAnyPackageFlags(PKG_ForDiffing) && IsRunningCookCommandlet())
		{
			if (!Context.GetTargetPlatform() || Context.GetTargetPlatform()->HasEditorOnlyData())
			{
				FString Value;
				const bool bRebuildSearchData = false;
				FSearchData SearchData = FFindInBlueprintSearchManager::Get().QuerySingleBlueprint((UBlueprint*)this, bRebuildSearchData);
				if (SearchData.IsValid())
				{
					Value = SearchData.Value;
				}

				Context.AddTag(FAssetRegistryTag(FBlueprintTags::FindInBlueprintsData, Value, FAssetRegistryTag::TT_Hidden));
			}
		}

		/*
		 * Restricting these tags to IsSaving because:
		 *	- UBlueprint is an asset in editor builds only.
		 *	- UBlueprintGeneratedClass is an asset in cooked builds only.
		 *	- Extended tags are only present in editor builds.
		 *
		 * See UBlueprintGeneratedClass::GetAssetRegistryTags.
		 */
		AActor* BlueprintCDOAsActor = Cast<AActor>(BlueprintCDO);
		if (BlueprintCDOAsActor)
		{
			FWorldPartitionActorDescUtils::AppendAssetDataTagsFromActor(BlueprintCDOAsActor, Context);
		}
	}
#endif
}

#if WITH_EDITOR
void UBlueprint::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	OutMetadata.Emplace(FBlueprintTags::ParentClassPath, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UBlueprint", "ParentClass", "Parent Class"))
		.SetTooltip(NSLOCTEXT("UBlueprint", "ParentClassTooltip", "The parent class of this blueprint.")));

	OutMetadata.Emplace(FBlueprintTags::NativeParentClassPath, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UBlueprint", "NativeParentClass", "Native Parent Class"))
		.SetTooltip(NSLOCTEXT("UBlueprint", "NativeParentClassTooltip", "The native parent class of this blueprint.")));

	OutMetadata.Emplace(FBlueprintTags::NumReplicatedProperties, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UBlueprint", "NumReplicatedProperties", "Replicated Properties"))
		.SetTooltip(NSLOCTEXT("UBlueprint", "NumReplicatedPropertiesTooltip", "The number of replicated properties in this blueprint.")));

	OutMetadata.Emplace(FBlueprintTags::IsDataOnly, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UBlueprint", "IsDataOnly", "Is Data Only"))
		.SetTooltip(NSLOCTEXT("UBlueprint", "IsDataOnlyTooltip", "Whether this is a data-only blueprint.")));

	OutMetadata.Emplace(FBlueprintTags::NumNativeComponents, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UBlueprint", "NumNativeComponents", "Native Components"))
		.SetTooltip(NSLOCTEXT("UBlueprint", "NumNativeComponentsTooltip", "The number of native components in this blueprint.")));

	OutMetadata.Emplace(FBlueprintTags::NumBlueprintComponents, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UBlueprint", "NumBlueprintComponents", "Blueprint Components"))
		.SetTooltip(NSLOCTEXT("UBlueprint", "NumBlueprintComponentsTooltip", "The number of blueprint components in this blueprint.")));
}
#endif

void UBlueprint::ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const
{
	Super::ThreadedPostLoadAssetRegistryTagsOverride(Context);
	PostLoadBlueprintAssetRegistryTags(Context);
}

void UBlueprint::PostLoadBlueprintAssetRegistryTags(const FAssetData& InAssetData, TArray<FAssetRegistryTag>& OutTagsAndValuesToUpdate)
{
	FPostLoadAssetRegistryTagsContext Context(InAssetData, OutTagsAndValuesToUpdate, IsInGameThread());
	PostLoadBlueprintAssetRegistryTags(Context);
}

void UBlueprint::PostLoadBlueprintAssetRegistryTags(FPostLoadAssetRegistryTagsContext& Context)
{
	auto FixTagValueShortClassName = [&Context](FName TagName, FAssetRegistryTag::ETagType TagType)
	{
		FString TagValue = Context.GetAssetData().GetTagValueRef<FString>(TagName);
		if (!TagValue.IsEmpty() && TagValue != TEXT("None"))
		{
			if (UClass::TryFixShortClassNameExportPath(TagValue, ELogVerbosity::Warning,
				TEXT("UBlueprint::PostLoadBlueprintAssetRegistryTags"), true /* bClearOnError */))
			{
				Context.AddTagToUpdate(FAssetRegistryTag(TagName, TagValue, TagType));
			}
		}
	};

	FixTagValueShortClassName(FBlueprintTags::GeneratedClassPath, FAssetRegistryTag::TT_Hidden);
	FixTagValueShortClassName(FBlueprintTags::ParentClassPath, FAssetRegistryTag::TT_Alphabetical);
	FixTagValueShortClassName(FBlueprintTags::NativeParentClassPath, FAssetRegistryTag::TT_Alphabetical);
}

FPrimaryAssetId UBlueprint::GetPrimaryAssetId() const
{
	// Forward to our Class, which will forward to CDO if needed
	// We use Generated instead of Skeleton because the CDO data is more accurate on Generated
	if (GeneratedClass && GeneratedClass->GetDefaultObject(false))
	{
		return GeneratedClass->GetPrimaryAssetId();
	}

	return FPrimaryAssetId();
}

FString UBlueprint::GetFriendlyName() const
{
	return GetName();
}

bool UBlueprint::AllowsDynamicBinding() const
{
	return FBlueprintEditorUtils::IsActorBased(this);
}

bool UBlueprint::SupportsInputEvents() const
{
	return ParentClass && ParentClass->IsChildOf(UObject::StaticClass());
}

struct FBlueprintInnerHelper
{
	template<typename TOBJ, typename TARR>
	static TOBJ* FindObjectByName(TARR& Array, const FName& TimelineName)
	{
		for(int32 i=0; i<Array.Num(); i++)
		{
			TOBJ* Obj = Array[i];
			if((NULL != Obj) && (Obj->GetFName() == TimelineName))
			{
				return Obj;
			}
		}
		return NULL;
	}
};

UActorComponent* UBlueprint::FindTemplateByName(const FName& TemplateName) const
{
	return FBlueprintInnerHelper::FindObjectByName<UActorComponent>(ComponentTemplates, TemplateName);
}

const UTimelineTemplate* UBlueprint::FindTimelineTemplateByVariableName(const FName& TimelineName) const
{
	const FName TimelineTemplateName = *UTimelineTemplate::TimelineVariableNameToTemplateName(TimelineName);
	const UTimelineTemplate* Timeline =  FBlueprintInnerHelper::FindObjectByName<const UTimelineTemplate>(Timelines, TimelineTemplateName);

	// >>> Backwards Compatibility:  VER_UE4_EDITORONLY_BLUEPRINTS
	if(Timeline)
	{
		ensure(Timeline->GetOuter() && Timeline->GetOuter()->IsA(UClass::StaticClass()));
	}
	else
	{
		Timeline = FBlueprintInnerHelper::FindObjectByName<const UTimelineTemplate>(Timelines, TimelineName);
		if(Timeline)
		{
			ensure(Timeline->GetOuter() && Timeline->GetOuter() == this);
		}
	}
	// <<< End Backwards Compatibility

	return Timeline;
}

UTimelineTemplate* UBlueprint::FindTimelineTemplateByVariableName(const FName& TimelineName)
{
	const FName TimelineTemplateName = *UTimelineTemplate::TimelineVariableNameToTemplateName(TimelineName);
	UTimelineTemplate* Timeline = FBlueprintInnerHelper::FindObjectByName<UTimelineTemplate>(Timelines, TimelineTemplateName);
	
	// >>> Backwards Compatibility:  VER_UE4_EDITORONLY_BLUEPRINTS
	if(Timeline)
	{
		ensure(Timeline->GetOuter() && Timeline->GetOuter()->IsA(UClass::StaticClass()));
	}
	else
	{
		Timeline = FBlueprintInnerHelper::FindObjectByName<UTimelineTemplate>(Timelines, TimelineName);
		if(Timeline)
		{
			ensure(Timeline->GetOuter() && Timeline->GetOuter() == this);
		}
	}
	// <<< End Backwards Compatibility

	return Timeline;
}

bool UBlueprint::ForceLoad(UObject* Obj)
{
	FLinkerLoad* Linker = Obj->GetLinker();
	if (Linker && !Obj->HasAnyFlags(RF_LoadCompleted))
	{
		UE::TUniqueLock LinkerLock(Linker->Mutex);
		Obj->SetFlags(RF_NeedLoad);
		Linker->Preload(Obj);
		return true;
	}
	return false;
}

void UBlueprint::ForceLoadMembers(UObject* InObject)
{
	if(const UBlueprint* Blueprint = Cast<UBlueprint>(InObject))
	{
		ForceLoadMembers(InObject, Blueprint);
		return;
	}

	if(const UClass* Class = Cast<UClass>(InObject))
	{
		ForceLoadMembers(InObject, Cast<UBlueprint>(Class->ClassGeneratedBy));
		return;
	}

	if(InObject->HasAnyFlags(RF_ClassDefaultObject))
	{
		if(const UClass* Class = InObject->GetClass())
		{
			ForceLoadMembers(InObject, Cast<UBlueprint>(Class->ClassGeneratedBy));
			return;
		}
	}

	ForceLoadMembers(InObject, nullptr);
}

void UBlueprint::ForceLoadMembers(UObject* InObject, const UBlueprint* InBlueprint)
{
	check(InObject);
	
	if(InObject && InBlueprint)
	{
		if(!InBlueprint->RequiresForceLoadMembers(InObject))
		{
			return;
		}
	}
	
	// Collect a list of all things this element owns
	TArray<UObject*> MemberReferences;
	FReferenceFinder ComponentCollector(MemberReferences, InObject, false, true, true, true);
	ComponentCollector.FindReferences(InObject);

	// Iterate over the list, and preload everything so it is valid for refreshing
	for (TArray<UObject*>::TIterator it(MemberReferences); it; ++it)
	{
		UObject* CurrentObject = *it;
		if (ForceLoad(CurrentObject))
		{
			ForceLoadMembers(CurrentObject, InBlueprint);
		}
	}
}

bool UBlueprint::ValidateGeneratedClass(const UClass* InClass)
{
	const UBlueprintGeneratedClass* GeneratedClass = Cast<const UBlueprintGeneratedClass>(InClass);
	if (!ensure(GeneratedClass))
	{
		return false;
	}
	const UBlueprint* Blueprint = GetBlueprintFromClass(GeneratedClass);
	if (!ensure(Blueprint))
	{
		return false;
	}

	for (auto CompIt = Blueprint->ComponentTemplates.CreateConstIterator(); CompIt; ++CompIt)
	{
		const UActorComponent* Component = (*CompIt);
		if (!ensure(Component && (Component->GetOuter() == GeneratedClass)))
		{
			return false;
		}
	}
	for (auto CompIt = GeneratedClass->ComponentTemplates.CreateConstIterator(); CompIt; ++CompIt)
	{
		const UActorComponent* Component = (*CompIt);
		if (!ensure(Component && (Component->GetOuter() == GeneratedClass)))
		{
			return false;
		}
	}

	for (auto CompIt = Blueprint->Timelines.CreateConstIterator(); CompIt; ++CompIt)
	{
		const UTimelineTemplate* Template = (*CompIt);
		if (!ensure(Template && (Template->GetOuter() == GeneratedClass)))
		{
			return false;
		}
	}
	for (auto CompIt = GeneratedClass->Timelines.CreateConstIterator(); CompIt; ++CompIt)
	{
		const UTimelineTemplate* Template = (*CompIt);
		if (!ensure(Template && (Template->GetOuter() == GeneratedClass)))
		{
			return false;
		}
	}

	if (const USimpleConstructionScript* SimpleConstructionScript = Blueprint->SimpleConstructionScript)
	{
		if (!ensure(SimpleConstructionScript->GetOuter() == GeneratedClass))
		{
			return false;
		}
	}
	if (const USimpleConstructionScript* SimpleConstructionScript = GeneratedClass->SimpleConstructionScript)
	{
		if (!ensure(SimpleConstructionScript->GetOuter() == GeneratedClass))
		{
			return false;
		}
	}

	if (const UInheritableComponentHandler* InheritableComponentHandler = Blueprint->InheritableComponentHandler)
	{
		if (!ensure(InheritableComponentHandler->GetOuter() == GeneratedClass))
		{
			return false;
		}
	}

	if (const UInheritableComponentHandler* InheritableComponentHandler = GeneratedClass->InheritableComponentHandler)
	{
		if (!ensure(InheritableComponentHandler->GetOuter() == GeneratedClass))
		{
			return false;
		}
	}

	return true;
}

void UBlueprint::BeginCacheForCookedPlatformData(const ITargetPlatform *TargetPlatform)
{
	Super::BeginCacheForCookedPlatformData(TargetPlatform);

	// Reset, in case data was previously cooked for another platform.
	ClearAllCachedCookedPlatformData();

	if (GeneratedClass && GeneratedClass->IsChildOf<AActor>())
	{
		int32 NumCookedComponents = 0;
		const double StartTime = FPlatformTime::Seconds();

		// Don't cook component data if the template won't be valid in the target context.
		auto ShouldCookBlueprintComponentTemplate = [TargetPlatform](const UActorComponent* InComponentTemplate) -> bool
		{
			if (InComponentTemplate)
			{
				return InComponentTemplate->NeedsLoadForTargetPlatform(TargetPlatform)
					&& (!TargetPlatform->IsClientOnly() || InComponentTemplate->NeedsLoadForClient())
					&& (!TargetPlatform->IsServerOnly() || InComponentTemplate->NeedsLoadForServer());
			}

			return false;
		};

		auto ShouldCookBlueprintComponentTemplateData = [](UBlueprintGeneratedClass* InBPGClass) -> bool
		{
			// Check to see if we should cook component data for the given class type.
			bool bResult = false;
			switch (GetDefault<UCookerSettings>()->BlueprintComponentDataCookingMethod)
			{
			case EBlueprintComponentDataCookingMethod::EnabledBlueprintsOnly:
				if (AActor* CDO = Cast<AActor>(InBPGClass->GetDefaultObject(false)))
				{
					bResult = CDO->ShouldCookOptimizedBPComponentData();
				}
				break;

			case EBlueprintComponentDataCookingMethod::AllBlueprints:
				bResult = true;
				break;

			case EBlueprintComponentDataCookingMethod::Disabled:
			default:
				break;
			}
			
			return bResult;
		};

		// Only cook component data if the setting is enabled and this is an Actor-based Blueprint class.
		if (UBlueprintGeneratedClass* BPGClass = CastChecked<UBlueprintGeneratedClass>(*GeneratedClass))
		{
			if (ShouldCookBlueprintComponentTemplateData(BPGClass))
			{
				// Cook all overridden SCS component node templates inherited from the parent class hierarchy.
				if (UInheritableComponentHandler* TargetInheritableComponentHandler = BPGClass->GetInheritableComponentHandler())
				{
					for (auto RecordIt = TargetInheritableComponentHandler->CreateRecordIterator(); RecordIt; ++RecordIt)
					{
						// Only generate cooked data if the target platform supports the template class type. Cooked data may already have been generated if the component was inherited from a nativized parent class.
						if (!RecordIt->CookedComponentInstancingData.bHasValidCookedData && ShouldCookBlueprintComponentTemplate(RecordIt->ComponentTemplate))
						{
							// Note: This will currently block until finished.
							// @TODO - Make this an async task so we can potentially cook instancing data for multiple components in parallel.
							FBlueprintEditorUtils::BuildComponentInstancingData(RecordIt->ComponentTemplate, RecordIt->CookedComponentInstancingData);
							++NumCookedComponents;
						}
					}
				}

				// Cook all SCS component templates that are owned by this class.
				if (BPGClass->SimpleConstructionScript)
				{
					for (auto Node : BPGClass->SimpleConstructionScript->GetAllNodes())
					{
						// Only generate cooked data if the target platform supports the template class type.
						if (ShouldCookBlueprintComponentTemplate(Node->ComponentTemplate))
						{
							// Note: This will currently block until finished.
							// @TODO - Make this an async task so we can potentially cook instancing data for multiple components in parallel.
							FBlueprintEditorUtils::BuildComponentInstancingData(Node->ComponentTemplate, Node->CookedComponentInstancingData);
							++NumCookedComponents;
						}
					}
				}

				// Cook all UCS/AddComponent node templates that are owned by this class.
				for (UActorComponent* ComponentTemplate : BPGClass->ComponentTemplates)
				{
					// Only generate cooked data if the target platform supports the template class type.
					if (ShouldCookBlueprintComponentTemplate(ComponentTemplate))
					{
						FBlueprintCookedComponentInstancingData& CookedComponentInstancingData = BPGClass->CookedComponentInstancingData.FindOrAdd(ComponentTemplate->GetFName());

						// Note: This will currently block until finished.
						// @TODO - Make this an async task so we can potentially cook instancing data for multiple components in parallel.
						FBlueprintEditorUtils::BuildComponentInstancingData(ComponentTemplate, CookedComponentInstancingData);
						++NumCookedComponents;
					}
				}

				// Flag that the BP class has cooked data to support fast path component instancing.
				BPGClass->bHasCookedComponentInstancingData = true;
			}
		}

		if (NumCookedComponents > 0)
		{
			UE_LOGF(LogBlueprint, Log, "%ls: Cooked %d component(s) in %.02g ms", *GetName(), NumCookedComponents, (FPlatformTime::Seconds() - StartTime) * 1000.0);
		}
	}
}

bool UBlueprint::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	// @TODO - Check async tasks for completion. For now just return TRUE since all tasks will currently block until finished.
	return true;
}

void UBlueprint::ClearAllCachedCookedPlatformData()
{
	Super::ClearAllCachedCookedPlatformData();

	if (UBlueprintGeneratedClass* BPGClass = Cast<UBlueprintGeneratedClass>(*GeneratedClass))
	{
		// Clear cooked data for overridden SCS component node templates inherited from the parent class hierarchy.
		if (UInheritableComponentHandler* TargetInheritableComponentHandler = BPGClass->GetInheritableComponentHandler())
		{
			for (auto RecordIt = TargetInheritableComponentHandler->CreateRecordIterator(); RecordIt; ++RecordIt)
			{
				RecordIt->CookedComponentInstancingData = FBlueprintCookedComponentInstancingData();
			}
		}

		// Clear cooked data for SCS component node templates.
		if (BPGClass->SimpleConstructionScript)
		{
			for (auto Node : BPGClass->SimpleConstructionScript->GetAllNodes())
			{
				Node->CookedComponentInstancingData = FBlueprintCookedComponentInstancingData();
			}
		}

		// Clear cooked data for UCS/AddComponent node templates.
		BPGClass->CookedComponentInstancingData.Empty();
	}
}

void UBlueprint::BeginDestroy()
{
	Super::BeginDestroy();

	FBlueprintEditorUtils::RemoveAllLocalBookmarks(this);

	// For each cached dependency, remove ourselves from its cached dependent set.
	for (const TWeakObjectPtr<UBlueprint>& DependencyReference : CachedDependencies)
	{
		if (UBlueprint* Dependency = DependencyReference.Get())
		{
			Dependency->CachedDependents.Remove(MakeWeakObjectPtr(this));
		}
	}
}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
bool UBlueprint::ShouldCookPropertyGuids() const
{
	switch (ShouldCookPropertyGuidsValue)
	{
	case EShouldCookBlueprintPropertyGuids::No:
		return false;

	case EShouldCookBlueprintPropertyGuids::Yes:
		return true;

	case EShouldCookBlueprintPropertyGuids::Inherit:
		if (const UBlueprint* ParentBlueprint = UBlueprint::GetBlueprintFromClass(ParentClass))
		{
			return ParentBlueprint->ShouldCookPropertyGuids();
		}
		break;
	}

	return false;
}

UBlueprint* UBlueprint::GetBlueprintFromClass(const UClass* InClass)
{
	UBlueprint* BP = NULL;
	if (InClass != NULL)
	{
		BP = Cast<UBlueprint>(InClass->ClassGeneratedBy);
	}
	return BP;
}

bool UBlueprint::GetBlueprintHierarchyFromClass(const UClass* InClass, TArray<UBlueprint*>& OutBlueprintParents)
{
	OutBlueprintParents.Reset();

	bool bNoErrors = true;
	const UClass* CurrentClass = InClass;
	while (UBlueprint* BP = UBlueprint::GetBlueprintFromClass(CurrentClass))
	{
		OutBlueprintParents.Add(BP);

		bNoErrors &= (BP->Status != BS_Error);

		// If valid, use stored ParentClass rather than the actual UClass::GetSuperClass(); handles the case when the class has not been recompiled yet after a reparent operation.
		if (BP->ParentClass)
		{
			CurrentClass = BP->ParentClass;
		}
		else
		{
			check(CurrentClass);
			CurrentClass = CurrentClass->GetSuperClass();
		}
	}

	return bNoErrors;
}
#endif

bool UBlueprint::GetBlueprintHierarchyFromClass(const UClass* InClass, TArray<UBlueprintGeneratedClass*>& OutBlueprintParents)
{
	OutBlueprintParents.Reset();

	bool bNoErrors = true;
	UBlueprintGeneratedClass* CurrentClass = Cast<UBlueprintGeneratedClass>(const_cast<UClass*>(InClass));
	while (CurrentClass)
	{
		OutBlueprintParents.Add(CurrentClass);

#if WITH_EDITORONLY_DATA
		UBlueprint* BP = UBlueprint::GetBlueprintFromClass(CurrentClass);

		if (BP)
		{
			bNoErrors &= (BP->Status != BS_Error);
		}

		// If valid, use stored ParentClass rather than the actual UClass::GetSuperClass(); handles the case when the class has not been recompiled yet after a reparent operation.
		if (BP && BP->ParentClass)
		{
			CurrentClass = Cast<UBlueprintGeneratedClass>(BP->ParentClass);
		}
		else
#endif // #if WITH_EDITORONLY_DATA
		{
			check(CurrentClass);
			CurrentClass = Cast<UBlueprintGeneratedClass>(CurrentClass->GetSuperClass());
		}
	}

	return bNoErrors;
}

bool UBlueprint::GetBlueprintHierarchyFromClass(const UClass* InClass, TArray<IBlueprintPropertyGuidProvider*>& OutBlueprintParents)
{
	OutBlueprintParents.Reset();

	bool bNoErrors = true;

	UBlueprintGeneratedClass* CurrentClass = Cast<UBlueprintGeneratedClass>(const_cast<UClass*>(InClass));
	while (CurrentClass)
	{
		IBlueprintPropertyGuidProvider* GuidProviderToAdd = CurrentClass;

#if WITH_EDITORONLY_DATA
		UBlueprint* BP = UBlueprint::GetBlueprintFromClass(CurrentClass);

		if (BP)
		{
			GuidProviderToAdd = BP;
			bNoErrors &= (BP->Status != BS_Error);
		}

		// If valid, use stored ParentClass rather than the actual UClass::GetSuperClass(); handles the case when the class has not been recompiled yet after a reparent operation.
		if (BP && BP->ParentClass)
		{
			CurrentClass = Cast<UBlueprintGeneratedClass>(BP->ParentClass);
		}
		else
#endif // #if WITH_EDITORONLY_DATA
		{
			check(CurrentClass);
			CurrentClass = Cast<UBlueprintGeneratedClass>(CurrentClass->GetSuperClass());
		}

		OutBlueprintParents.Add(GuidProviderToAdd);
	}

	return bNoErrors;
}

#if WITH_EDITOR
bool UBlueprint::IsBlueprintHierarchyErrorFree(const UClass* InClass)
{
	const UClass* CurrentClass = InClass;
	while (UBlueprint* BP = UBlueprint::GetBlueprintFromClass(CurrentClass))
	{
		if(BP->Status == BS_Error)
		{
			return false;
		}

		// If valid, use stored ParentClass rather than the actual UClass::GetSuperClass(); handles the case when the class has not been recompiled yet after a reparent operation.
		if(const UClass* ParentClass = BP->ParentClass)
		{
			CurrentClass = ParentClass;
		}
		else
		{
			check(CurrentClass);
			CurrentClass = CurrentClass->GetSuperClass();
		}
	}

	return true;
}
#endif

FName UBlueprint::FindBlueprintPropertyNameFromGuid(const FGuid& PropertyGuid) const
{
#if WITH_EDITORONLY_DATA
	for (const FBPVariableDescription& BPVarDesc : NewVariables)
	{
		if (BPVarDesc.VarGuid == PropertyGuid)
		{
			return BPVarDesc.VarName;
		}
	}

	for (const FBPVariableDescription& BPVarDesc : GeneratedVariables)
	{
		if (BPVarDesc.VarGuid == PropertyGuid)
		{
			return BPVarDesc.VarName;
		}
	}
#endif

	return NAME_None;
}

FGuid UBlueprint::FindBlueprintPropertyGuidFromName(const FName PropertyName) const
{
#if WITH_EDITORONLY_DATA
	for (const FBPVariableDescription& BPVarDesc : NewVariables)
	{
		if (BPVarDesc.VarName == PropertyName)
		{
			return BPVarDesc.VarGuid;
		}
	}

	for (const FBPVariableDescription& BPVarDesc : GeneratedVariables)
	{
		if (BPVarDesc.VarName == PropertyName)
		{
			return BPVarDesc.VarGuid;
		}
	}
#endif

	return FGuid();
}

ETimelineSigType UBlueprint::GetTimelineSignatureForFunctionByName(const FName& FunctionName, const FName& ObjectPropertyName)
{
	check(SkeletonGeneratedClass != NULL);
	
	UClass* UseClass = SkeletonGeneratedClass;

	// If an object property was specified, find the class of that property instead
	if(ObjectPropertyName != NAME_None)
	{
		FObjectPropertyBase* ObjProperty = FindFProperty<FObjectPropertyBase>(SkeletonGeneratedClass, ObjectPropertyName);
		if(ObjProperty == NULL)
		{
			UE_LOGF(LogBlueprint, Log, "GetTimelineSignatureForFunction: Object Property '%ls' not found.", *ObjectPropertyName.ToString());
			return ETS_InvalidSignature;
		}

		UseClass = ObjProperty->PropertyClass;
	}

	UFunction* Function = FindUField<UFunction>(UseClass, FunctionName);
	if(Function == NULL)
	{
		UE_LOGF(LogBlueprint, Log, "GetTimelineSignatureForFunction: Function '%ls' not found in class '%ls'.", *FunctionName.ToString(), *UseClass->GetName());
		return ETS_InvalidSignature;
	}

	return UTimelineComponent::GetTimelineSignatureForFunction(Function);


	//UE_LOGF(LogBlueprint, Log, "GetTimelineSignatureForFunction: No SkeletonGeneratedClass in Blueprint '%ls'.", *GetName());
	//return ETS_InvalidSignature;
}

FString UBlueprint::GetDesc(void)
{
	FString BPType;
	switch (BlueprintType)
	{
		case BPTYPE_MacroLibrary:
			BPType = TEXT("macros for");
			break;
		case BPTYPE_Const:
			BPType = TEXT("const extends");
			break;
		case BPTYPE_Interface:
			// Always extends interface, so no extraneous information needed
			BPType = TEXT("");
			break;
		default:
			BPType = TEXT("extends");
			break;
	}
	const FString ResultString = FString::Printf(TEXT("%s %s"), *BPType, *ParentClass->GetName());

	return ResultString;
}

bool UBlueprint::NeedsLoadForClient() const
{
	return false;
}

bool UBlueprint::NeedsLoadForServer() const
{
	return false;
}

bool UBlueprint::NeedsLoadForEditorGame() const
{
	return true;
}

bool UBlueprint::HasNonEditorOnlyReferences() const
{
	// The this->BlueprintGeneratedClass is reference that we need to mark as UsedInGame,
	// despite UBlueprint being editor-only due to NeedsLoadForClient,NeedsLoadForServer == false
	return true;
}

void UBlueprint::TagSubobjects(EObjectFlags NewFlags)
{
	Super::TagSubobjects(NewFlags);

	if (GeneratedClass && !GeneratedClass->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS))
	{
		GeneratedClass->SetFlags(NewFlags);
		GeneratedClass->TagSubobjects(NewFlags);
	}

	if (SkeletonGeneratedClass && SkeletonGeneratedClass != GeneratedClass && !SkeletonGeneratedClass->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS))
	{
		SkeletonGeneratedClass->SetFlags(NewFlags);
		SkeletonGeneratedClass->TagSubobjects(NewFlags);
	}
}

void UBlueprint::GetAllGraphs(TArray<UEdGraph*>& Graphs) const
{
#if WITH_EDITORONLY_DATA
	for (int32 i = 0; i < FunctionGraphs.Num(); ++i)
	{
		UEdGraph* Graph = FunctionGraphs[i];
		if(Graph)
		{
			Graphs.Add(Graph);
			Graph->GetAllChildrenGraphs(Graphs);
		}
	}
	for (int32 i = 0; i < MacroGraphs.Num(); ++i)
	{
		UEdGraph* Graph = MacroGraphs[i];
		if(Graph)
		{
			Graphs.Add(Graph);
			Graph->GetAllChildrenGraphs(Graphs);
		}
	}

	for (int32 i = 0; i < UbergraphPages.Num(); ++i)
	{
		UEdGraph* Graph = UbergraphPages[i];
		if(Graph)
		{
			Graphs.Add(Graph);
			Graph->GetAllChildrenGraphs(Graphs);
		}
	}

	for (int32 i = 0; i < DelegateSignatureGraphs.Num(); ++i)
	{
		UEdGraph* Graph = DelegateSignatureGraphs[i];
		if(Graph)
		{
			Graphs.Add(Graph);
			Graph->GetAllChildrenGraphs(Graphs);
		}
	}

	for (int32 BPIdx=0; BPIdx<ImplementedInterfaces.Num(); BPIdx++)
	{
		const FBPInterfaceDescription& InterfaceDesc = ImplementedInterfaces[BPIdx];
		for (int32 GraphIdx = 0; GraphIdx < InterfaceDesc.Graphs.Num(); GraphIdx++)
		{
			UEdGraph* Graph = InterfaceDesc.Graphs[GraphIdx];
			if(Graph)
			{
				Graphs.Add(Graph);
				Graph->GetAllChildrenGraphs(Graphs);
			}
		}
	}

	for (const UBlueprintExtension* Extension : GetExtensions())
	{
		if (Extension != nullptr)
		{
			TArray<UEdGraph*> ExtensionGraphs;
			Extension->GetAllGraphs(ExtensionGraphs);
			for (int32 i = 0; i < ExtensionGraphs.Num(); ++i)
			{
				UEdGraph* Graph = ExtensionGraphs[i];
				if(Graph)
				{
					Graphs.Add(Graph);
					Graph->GetAllChildrenGraphs(Graphs);
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
bool UBlueprint::ChangeOwnerOfTemplates()
{
	struct FUniqueNewNameHelper
	{
	private:
		FString NewName;
		bool isValid;
	public:
		FUniqueNewNameHelper(const FString& Name, UObject* Outer) : isValid(false)
		{
			const uint32 Hash = FCrc::StrCrc32<TCHAR>(*Name);
			NewName = FString::Printf(TEXT("%s__%08X"), *Name, Hash);
			isValid = IsUniqueObjectName(FName(*NewName), Outer);
			if (!isValid)
			{
				check(Outer);
				UE_LOGF(LogBlueprint, Warning, "ChangeOwnerOfTemplates: Cannot generate a deterministic new name. Old name: %ls New outer: %ls", *Name, *Outer->GetName());
			}
		}

		const TCHAR* Get() const
		{
			return isValid ? *NewName : NULL;
		}
	};

	UBlueprintGeneratedClass* BPGClass = Cast<UBlueprintGeneratedClass>(*GeneratedClass);
	bool bIsStillStale = false;
	if (BPGClass)
	{
		// >>> Backwards Compatibility:  VER_UE4_EDITORONLY_BLUEPRINTS
		bool bMigratedOwner = false;
		TSet<class UCurveBase*> Curves;
		for( auto CompIt = ComponentTemplates.CreateIterator(); CompIt; ++CompIt )
		{
			UActorComponent* Component = (*CompIt);
			if (Component)
			{
				if (Component->GetOuter() == this)
				{
					const bool bRenamed = Component->Rename(*Component->GetName(), BPGClass, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
					ensure(bRenamed);
					bIsStillStale |= !bRenamed;
					bMigratedOwner = true;
				}
				if (auto TimelineComponent = Cast<UTimelineComponent>(Component))
				{
					TimelineComponent->GetAllCurves(Curves);
				}
			}
		}

		for( auto CompIt = Timelines.CreateIterator(); CompIt; ++CompIt )
		{
			UTimelineTemplate* Template = (*CompIt);
			if (Template)
			{
				if(Template->GetOuter() == this)
				{
					const FName OldTemplateName = Template->GetFName();
					ensure(!OldTemplateName.ToString().EndsWith(UTimelineTemplate::TemplatePostfix));
					const bool bRenamed = Template->Rename(*UTimelineTemplate::TimelineVariableNameToTemplateName(Template->GetFName()), BPGClass, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
					ensure(bRenamed);
					bIsStillStale |= !bRenamed;
					ensure(OldTemplateName == Template->GetVariableName());
					bMigratedOwner = true;
				}
				Template->GetAllCurves(Curves);
			}
		}
		for (auto Curve : Curves)
		{
			if (Curve && (Curve->GetOuter() == this))
			{
				const bool bRenamed = Curve->Rename(FUniqueNewNameHelper(Curve->GetName(), BPGClass).Get(), BPGClass, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
				ensure(bRenamed);
				bIsStillStale |= !bRenamed;
			}
		}

		if(USimpleConstructionScript* SCS = SimpleConstructionScript)
		{
			if(SCS->GetOuter() == this)
			{
				const bool bRenamed = SCS->Rename(FUniqueNewNameHelper(SCS->GetName(), BPGClass).Get(), BPGClass, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
				ensure(bRenamed);
				bIsStillStale |= !bRenamed;
				bMigratedOwner = true;
			}

			for (USCS_Node* SCSNode : SCS->GetAllNodes())
			{
				UActorComponent* Component = SCSNode ? ToRawPtr(SCSNode->ComponentTemplate) : NULL;
				if (Component && Component->GetOuter() == this)
				{
					const bool bRenamed = Component->Rename(FUniqueNewNameHelper(Component->GetName(), BPGClass).Get(), BPGClass, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
					ensure(bRenamed);
					bIsStillStale |= !bRenamed;
					bMigratedOwner = true;
				}
			}
		}

		if (bMigratedOwner)
		{
			if( !HasAnyFlags( RF_Transient ))
			{
				// alert the user that blueprints gave been migrated and require re-saving to enable them to locate and fix them without nagging them.
				FMessageLog("BlueprintLog").Warning( FText::Format( NSLOCTEXT( "Blueprint", "MigrationWarning", "Blueprint {0} has been migrated and requires re-saving to avoid import errors" ), FText::FromString( *GetName() )));

				if( GetDefault<UEditorLoadingSavingSettings>()->bDirtyMigratedBlueprints )
				{
					UPackage* BPPackage = GetOutermost();

					if( BPPackage )
					{
						BPPackage->SetDirtyFlag( true );
					}
				}
			}

			BPGClass->ComponentTemplates = ComponentTemplates;
			BPGClass->Timelines = Timelines;
			BPGClass->SimpleConstructionScript = SimpleConstructionScript;
		}
		// <<< End Backwards Compatibility
	}
	else
	{
		UE_LOGF(LogBlueprint, Log, "ChangeOwnerOfTemplates: No BlueprintGeneratedClass in %ls", *GetName());
	}
	return !bIsStillStale;
}

#if WITH_EDITOR
bool UBlueprint::Modify(bool bAlwaysMarkDirty)
{
	bCachedDependenciesUpToDate = false;
	return Super::Modify(bAlwaysMarkDirty);
}
#endif

void UBlueprint::GatherDependencies(TSet<TWeakObjectPtr<UBlueprint>>& InDependencies) const
{

}

void UBlueprint::ReplaceDeprecatedNodes()
{
	TArray<UEdGraph*> Graphs;
	GetAllGraphs(Graphs);

	for ( auto It = Graphs.CreateIterator(); It; ++It )
	{
		UEdGraph* const Graph = *It;
		const UEdGraphSchema* Schema = Graph->GetSchema();
		Schema->BackwardCompatibilityNodeConversion(Graph, true);
	}
}

void UBlueprint::ClearEditorReferences()
{
	FKismetEditorUtilities::OnBlueprintUnloaded.Broadcast(this);
	if (UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(GeneratedClass))
	{
		FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.Broadcast(BPGC);
	}
}

UInheritableComponentHandler* UBlueprint::GetInheritableComponentHandler(bool bCreateIfNecessary)
{
	static const FBoolConfigValueHelper EnableInheritableComponents(TEXT("Kismet"), TEXT("bEnableInheritableComponents"), GEngineIni);
	if (!EnableInheritableComponents)
	{
		return nullptr;
	}

	if (!InheritableComponentHandler && bCreateIfNecessary)
	{
		UBlueprintGeneratedClass* BPGC = CastChecked<UBlueprintGeneratedClass>(GeneratedClass);
		ensure(!BPGC->InheritableComponentHandler);
		InheritableComponentHandler = BPGC->GetInheritableComponentHandler(true);
	}
	return InheritableComponentHandler;
}


EDataValidationResult UBlueprint::IsDataValid(FDataValidationContext& Context) const
{
	const UObject* GeneratedClassCDO = GeneratedClass ? GeneratedClass->GetDefaultObject() : nullptr;
	EDataValidationResult IsValid = GeneratedClassCDO ? GeneratedClassCDO->IsDataValid(Context) : EDataValidationResult::Invalid;
	IsValid = (IsValid == EDataValidationResult::NotValidated) ? EDataValidationResult::Valid : IsValid;

	if (SimpleConstructionScript)
	{
		EDataValidationResult IsSCSValid = SimpleConstructionScript->IsDataValid(Context);
		IsValid = CombineDataValidationResults(IsValid, IsSCSValid);
	}

	if (InheritableComponentHandler)
	{
		EDataValidationResult IsICHValid = InheritableComponentHandler->IsDataValid(Context);
		IsValid = CombineDataValidationResults(IsValid, IsICHValid);
	}

	for (const UActorComponent* Component : ComponentTemplates)
	{
		if (Component)
		{
			EDataValidationResult IsComponentValid = Component->IsDataValid(Context);
			IsValid = CombineDataValidationResults(IsValid, IsComponentValid);
		}
	}

	for (const UTimelineTemplate* Timeline : Timelines)
	{
		if (Timeline)
		{
			EDataValidationResult IsTimelineValid = Timeline->IsDataValid(Context);
			IsValid = CombineDataValidationResults(IsValid, IsTimelineValid);
		}
	}

	return IsValid;
}

bool UBlueprint::FindDiffs(const UBlueprint* OtherBlueprint, FDiffResults& Results) const
{
	return false;
}

FName UBlueprint::GetFunctionNameFromClassByGuid(const UClass* InClass, const FGuid FunctionGuid)
{
	return FBlueprintEditorUtils::GetFunctionNameFromClassByGuid(InClass, FunctionGuid);
}

bool UBlueprint::GetFunctionGuidFromClassByFieldName(const UClass* InClass, const FName FunctionName, FGuid& FunctionGuid)
{
	return FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(InClass, FunctionName, FunctionGuid);
}

UEdGraph* UBlueprint::GetLastEditedUberGraph() const
{
	for ( int32 LastEditedIndex = LastEditedDocuments.Num() - 1; LastEditedIndex >= 0; LastEditedIndex-- )
	{
		if ( UObject* Obj = LastEditedDocuments[LastEditedIndex].EditedObjectPath.ResolveObject() )
		{
			if ( UEdGraph* Graph = Cast<UEdGraph>(Obj) )
			{
				for ( int32 GraphIndex = 0; GraphIndex < UbergraphPages.Num(); GraphIndex++ )
				{
					if ( Graph == UbergraphPages[GraphIndex] )
					{
						return UbergraphPages[GraphIndex];
					}
				}
			}
		}
	}

	if ( UbergraphPages.Num() > 0 )
	{
		return UbergraphPages[0];
	}

	return nullptr;
}

#if WITH_EDITOR

UClass* UBlueprint::GetBlueprintParentClassFromAssetTags(const FAssetData& BlueprintAsset)
{
	UClass* ParentClass = nullptr;
	FString ParentClassName;
	if(!BlueprintAsset.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
	{
		BlueprintAsset.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
	}
	
	if(!ParentClassName.IsEmpty())
	{
		UObject* Outer = nullptr;
		ResolveName(Outer, ParentClassName, false, false);
		ParentClass = FindObject<UClass>(Outer, *ParentClassName);
	}
	
	return ParentClass;
}

#endif

void UBlueprint::BroadcastChanged()
{
	if(bBeingCompiled)
	{
		return;
	}

	ChangedEvent.Broadcast(this);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

TArrayView<const TObjectPtr<UBlueprintExtension>> UBlueprint::GetExtensions() const
{
	return Extensions;
}

int32 UBlueprint::AddExtension(const TObjectPtr<UBlueprintExtension>& InExtension)
{
	int32 Index = Extensions.Add(InExtension);
	OnExtensionAdded.Broadcast(InExtension);
	return Index;
}

int32 UBlueprint::RemoveExtension(const TObjectPtr<UBlueprintExtension>& InExtension)
{
	int32 NumRemoved = Extensions.RemoveSingleSwap(InExtension);
	if (NumRemoved > 0)
	{
		OnExtensionRemoved.Broadcast(InExtension);
	}
	return NumRemoved;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UBlueprint::GeneratedClassPreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext)
{
}

void UBlueprint::GeneratedClassPostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
}

#endif //WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UBlueprint::LoadModulesRequiredForCompilation()
{
	static const FName KismetCompilerModuleName("KismetCompiler");
	static const FName MovieSceneToolsModuleName("MovieSceneTools");

	FModuleManager::Get().LoadModule(KismetCompilerModuleName);
	FModuleManager::Get().LoadModule(MovieSceneToolsModuleName);
}
#endif //WITH_EDITORONLY_DATA
