// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintFunctionLibrary)

#if WITH_EDITOR
#endif

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintFuncLibrary, Log, All);

UBlueprintFunctionLibrary::UBlueprintFunctionLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UBlueprintFunctionLibrary::GetFunctionCallspace(UFunction* Function, FFrame* Stack)
{
	return GEngine->GetGlobalFunctionCallspace(Function, this, Stack);
}


