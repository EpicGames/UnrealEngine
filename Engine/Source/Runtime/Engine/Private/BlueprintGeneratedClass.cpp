// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/BlueprintGeneratedClass.h"

#include "Containers/RingBuffer.h"
#include "Containers/VersePath.h"
#include "Blueprint/BlueprintSupport.h"
#include "HAL/IConsoleManager.h"
#include "EngineLogs.h"
#include "Stats/StatsMisc.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/CoreNet.h"
#include "UObject/CoreRedirects.h"
#include "UObject/ObjectSaveContext.h"
#include "Serialization/ObjectWriter.h"
#include "Curves/CurveFloat.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Components/TimelineComponent.h"
#include "Engine/TimelineTemplate.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/SCS_Node.h"
#include "Engine/InheritableComponentHandler.h"
#include "IAssetRegistryTagProviderInterface.h"
#include "IFieldNotificationClassDescriptor.h"
#include "INotifyFieldValueChanged.h"
#include "Misc/ConfigCacheIni.h"
#include "Net/Core/PushModel/PushModel.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ImportExportCollector.h"
#include "UObject/Package.h" // IWYU pragma: keep
#include "UObject/PrimaryAssetId.h"
#include "UObject/SparseClassDataUtils.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "AutoRTFM.h"
#include "Serialization/AsyncPackageLoader.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintGeneratedClass)

#if WITH_EDITOR
#include "CookerSettings.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BlueprintCompilationManager.h"
#include "Engine/LevelScriptBlueprint.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "UObject/CookedMetaData.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/UObjectGlobals.h"
extern UNREALED_API class UEditorEngine* GEditor;
#else
#include "UObject/LinkerLoad.h"
#endif //WITH_EDITOR

DEFINE_STAT(STAT_PersistentUberGraphFrameMemory);
DEFINE_STAT(STAT_BPCompInstancingFastPathMemory);

static int32 GBlueprintNativePropertyInitFastPathDisabled = 0;
static FAutoConsoleVariableRef CVarBlueprintNativePropertyInitFastPathDisabled(
	TEXT("bp.NativePropertyInitFastPathDisabled"),
	GBlueprintNativePropertyInitFastPathDisabled,
	TEXT("Disable the native property initialization fast path."),
	ECVF_Default
);

static int32 GBlueprintComponentInstancingFastPathDisabled = 0;
static FAutoConsoleVariableRef CVarBlueprintComponentInstancingFastPathDisabled(
	TEXT("bp.ComponentInstancingFastPathDisabled"),
	GBlueprintComponentInstancingFastPathDisabled,
	TEXT("Disable the Blueprint component instancing fast path."),
	ECVF_Default
);

#if WITH_EDITOR
static int32 GBlueprintDefaultSubobjectValidationDisabled = 1;
static FAutoConsoleVariableRef CVarBlueprintDefaultSubobjectValidationDisabled(
	TEXT("bp.DefaultSubobjectValidationDisabled"),
	GBlueprintDefaultSubobjectValidationDisabled,
	TEXT("Disable Blueprint class default subobject validation at editor load/save time."),
	ECVF_Default
);
#endif	// WITH_EDITOR

#if WITH_ADDITIONAL_CRASH_CONTEXTS
struct BPGCBreadcrumbsParams
{
	UObject& Object;
	UBlueprintGeneratedClass& BPGC;
	uint32 ThreadId;
};

// Called during a crash: dynamic memory allocations may not be reliable here.
void WriteBPGCBreadcrumbs(FCrashContextExtendedWriter& Writer, const BPGCBreadcrumbsParams& Params)
{
	constexpr int32 MAX_DATA_SIZE = 1024;
	constexpr int32 MAX_FILENAME_SIZE = 64;
	constexpr int32 MAX_THREADS_TO_LOG = 16;

	static int32 ThreadCount = 0;

	// In the unlikely case that there are too many threads that reported a BPGC-related crash, we'll just ignore the excess.
	// Theoretically, there should be enough information written by the remaining threads.
	if (ThreadCount < MAX_THREADS_TO_LOG)
	{
		// Note: TStringBuilder *can* potentially allocate dynamic memory if its inline storage is exceeded.
		// In practice, we stay under the current threshold by only recording the minimum amount data that we need.
		TStringBuilder<MAX_DATA_SIZE> Builder;

		Params.Object.GetPathName(nullptr, Builder);
		Builder.AppendChar(TEXT('\n'));
		Params.BPGC.GetPathName(nullptr, Builder);
		Builder.AppendChar(TEXT('\n'));

		TStringBuilder<MAX_FILENAME_SIZE> Filename;

		Filename.Appendf(TEXT("BPGCBreadcrumb_%u"), Params.ThreadId);

		Writer.AddString(Filename.ToString(), Builder.ToString());
	}

	++ThreadCount;
}
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS

namespace UE::Runtime::Engine::Private
{
	struct FBlueprintGeneratedClassUtils
	{
		static UClass* FindFirstNativeClassInHierarchy(UClass* InClass)
		{
			UClass* CurrentClass = InClass;
			while (CurrentClass && !CurrentClass->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
			{
				CurrentClass = CurrentClass->GetSuperClass();
			}

			return CurrentClass;
		}

		static bool RequiresCompleteValueForPostConstruction(FProperty* Property)
		{
			static TSet<FProperty*> PropertiesRequiringCompleteValueInitialization;

			static bool bIsInitialized = false;
#if WITH_EDITOR
			static bool bNeedsDelegateRegistration = true;
			if (bNeedsDelegateRegistration)
			{
				FCoreUObjectDelegates::ReloadCompleteDelegate.AddLambda([](EReloadCompleteReason Reason)
				{
					switch (Reason)
					{
					case EReloadCompleteReason::HotReloadManual:
					case EReloadCompleteReason::HotReloadAutomatic:
					{
						// Re-initialize after hot reload in editor context.
						bIsInitialized = false;
						PropertiesRequiringCompleteValueInitialization.Reset();
					}
					break;

					default:
						break;
					}
				});

				bNeedsDelegateRegistration = false;
			}
#endif	// WITH_EDITOR

			if (!bIsInitialized && GConfig)
			{
				static constexpr TCHAR ConfigSection[] = TEXT("/Script/Engine.BlueprintGeneratedClass");
				static constexpr TCHAR ConfigKeyName[] = TEXT("RequiresCompleteValueForPostConstruction");

				// List of native class-owned properties that require complete values for comparison when generating the post-construction
				// property list due to how they are used. These properties can't be further reduced into a list of subfields (e.g. structs).
				TArray<FString> RequiresCompleteValueForPostConstruction_Value;
				if (GConfig->GetArray(ConfigSection, ConfigKeyName, RequiresCompleteValueForPostConstruction_Value, GEngineIni))
				{
					PropertiesRequiringCompleteValueInitialization.Reserve(RequiresCompleteValueForPostConstruction_Value.Num());

					for (const FString& PropertyPathString : RequiresCompleteValueForPostConstruction_Value)
					{
						TFieldPath<FProperty> PropertyPath;
						PropertyPath.Generate(*PropertyPathString);
						if (FProperty* ResolvedProperty = PropertyPath.Get())
						{
							check(ResolvedProperty->IsNative());
							PropertiesRequiringCompleteValueInitialization.Add(ResolvedProperty);
						}
					}
				}

				bIsInitialized = true;
			}

			return PropertiesRequiringCompleteValueInitialization.Contains(Property);
		}

		static bool ShouldInitializePropertyDuringPostConstruction(const FProperty& Property)
		{
			const UClass* OwnerClass = Property.GetOwnerClass();
			const bool bIsConfigProperty = Property.HasAnyPropertyFlags(CPF_Config) && !(OwnerClass && OwnerClass->HasAnyClassFlags(CLASS_PerObjectConfig));
			const bool bIsTransientProperty = Property.HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient);
			const bool bIsForcePostConstructLink = Property.HasAnyPropertyFlags(CPF_ForcePostConstructLink);

			// Skip config and forced post construct link properties as they're already in the PostConstructLink chain. Also skip transient properties if they contain a reference to an instanced subobjects (as those should not be initialized from defaults).
			if (!bIsConfigProperty && !bIsForcePostConstructLink &&(!bIsTransientProperty || !Property.ContainsInstancedObjectProperty()))
			{
				return true;
			}

			return false;
		}

#if WITH_EDITOR
	protected:
		static void ValidateObjectPropertyValue(UObject* InOuter, const FObjectProperty* InProperty, void* InValuePtr, const void* InDefValuePtr)
		{
			check(InProperty);

			// Get the reference assigned to the value address for the given property.
			UObject* ObjValue = InProperty->GetObjectPropertyValue(InValuePtr);
			if (!ObjValue)
			{
				// If the current reference value is NULL, grab the reference at the default value address for the same property.
				ObjValue = InProperty->GetObjectPropertyValue(InDefValuePtr);
				if (ObjValue && ObjValue->IsDefaultSubobject() && ObjValue->HasAllFlags(RF_DefaultSubObject))
				{
					check(InOuter);

					// Attempt to find a matching instanced DSO within the current outer scope.
					UObject* CurrentValue = InOuter->GetDefaultSubobjectByName(ObjValue->GetFName());
					if (CurrentValue)
					{
						// In some cases, we might find a matching subobject instance that doesn't have the flag set to indicate that it was also
						// instanced at construction time as a default subobject. Only fix up the value here if the instance also has that flag.
						if (CurrentValue->HasAllFlags(RF_DefaultSubObject))
						{
							UE_LOGF(LogBlueprint, Warning, "%ls: Detected a NULL reference value for the class member named (%ls). Changes to this property may not be restored on load. Check to see if any changes need to be re-applied, then re-save the asset to fix this warning.", *InOuter->GetPathName(), *InProperty->GetName());

							// If the default reference is a non-NULL DSO, then the current container's reference should also be non-NULL. However,
							// we want a reference to a matching subobject that exists within the current container. For DSOs, this should have
							// been instanced at construction time (because we will have also run the container type's native constructor), but it's
							// possible that this field has lost the reference somewhere along the way (e.g. at serialization time). So in order to
							// ensure that we at least have a valid instanced DSO referenced by the property, reassign it to the current instance.
							// Note that the current instance may not have been serialized if the reference was already NULL at save time, so this
							// may result in data loss on load. However, this at least allows the object to be fixed up
							InProperty->SetObjectPropertyValue(InValuePtr, CurrentValue);
						}

						// No need to validate nested DSOs here - we've simply returned the field to the initialized state of the container object.
					}
					else
					{
						// Could not find a matching DSO instance within the current container's scope; warn about this, but leave it set to NULL.
						// We're not going to create a new instance here, because if we're in this situation, then it means the DSO was not
						// instanced for the container object at construction time, which would occur for example if we started allowing users
						// to mark inherited components as optional at the Blueprint editor level. So we'd want to determine why that occurred.
						// @todo - If Blueprints ever add support to mark DSOs as optional at the editor level, we'll then need to revisit this.
						UE_LOGF(LogBlueprint, Warning, "%ls: Missing a default subobject instance named \'%ls\'. This should have been instanced at construction time for \'%ls\'.", *InOuter->GetPathName(), *ObjValue->GetName(), *InProperty->GetOwnerStruct()->GetName());
					}
				}
			}
			else if(ObjValue->IsDefaultSubobject())
			{
				// If the current value is a default subobject, recursively validate any nested DSOs.
				ValidateDefaultSubobjects(ObjValue);
			}
		}

		static void ValidateInstancedObjectProperty(UObject* InOuter, const FProperty* InProperty, void* InDataPtr, const void* InDefaultDataPtr)
		{
			// It's possible for reference properties to be declared as a fixed array, so iterate over the fixed size (generally just one).
			for (int32 ArrayIdx = 0; ArrayIdx < InProperty->ArrayDim; ++ArrayIdx)
			{
				if (const FObjectProperty* ObjProp = CastField<FObjectProperty>(InProperty))
				{
					// For a scalar reference type, validate the current value against the default value, and ensure that they match.
					UObject* ObjValuePtr = ObjProp->ContainerPtrToValuePtr<UObject>(InDataPtr, ArrayIdx);
					const UObject* DefValuePtr = ObjProp->ContainerPtrToValuePtr<UObject>(InDefaultDataPtr, ArrayIdx);
					ValidateObjectPropertyValue(InOuter, ObjProp, ObjValuePtr, DefValuePtr);
				}
				else if (const FStructProperty* StructProp = CastField<FStructProperty>(InProperty))
				{
					// Recurse into struct properties, in case any members are assigned to a top-level DSO owned by the input object.
					ValidateDefaultSubobjects_Inner(
						InOuter,
						StructProp->Struct,
						StructProp->ContainerPtrToValuePtr<void>(InDataPtr, ArrayIdx),
						StructProp->ContainerPtrToValuePtr<void>(InDefaultDataPtr, ArrayIdx));
				}
				else if (const FArrayProperty* ArrProp = CastField<FArrayProperty>(InProperty))
				{
					// For array types, validate each element's value against the default value, and ensure that all DSO elements match up.
					// Note that it's possible for the default value to be larger or smaller than the current value in terms of the element
					// count; in that case, we assume DSOs can't be removed at the Blueprint level, so the default will always include them.
					FScriptArrayHelper_InContainer ArrValue(ArrProp, InDataPtr, ArrayIdx);
					FScriptArrayHelper_InContainer DefValue(ArrProp, InDefaultDataPtr, ArrayIdx);
					for (int32 ValueIdx = 0; ValueIdx < ArrValue.Num() && ValueIdx < DefValue.Num(); ++ValueIdx)
					{
						ValidateInstancedObjectProperty(InOuter, ArrProp->Inner, ArrValue.GetRawPtr(ValueIdx), DefValue.GetRawPtr(ValueIdx));
					}
				}
				else if (const FSetProperty* SetProp = CastField<FSetProperty>(InProperty))
				{
					// For set containers, validate each element's value against the default value, and ensure that all DSO elements match up.
					// As with arrays, we must also consider that the element counts may differ between current and default, but that all DSOs
					// are at least always present in the default container, and match up with the elements in the current set container value.
					FScriptSetHelper_InContainer SetValue(SetProp, InDataPtr, ArrayIdx);
					FScriptSetHelper_InContainer DefValue(SetProp, InDefaultDataPtr, ArrayIdx);
					for (int32 ValueIdx = 0; ValueIdx < SetValue.Num() && ValueIdx < DefValue.Num(); ++ValueIdx)
					{
						ValidateInstancedObjectProperty(InOuter, SetProp->ElementProp, SetValue.GetElementPtr(ValueIdx), DefValue.GetElementPtr(ValueIdx));
					}
				}
				else if (const FMapProperty* MapProp = CastField<FMapProperty>(InProperty))
				{
					// For map containers, validate each pair's value against the default value, and ensure that all DSO elements match up.
					// As above, we consider that the number of pairs in the map may differ between current and default, but that all DSOs
					// are at least always present in the default container, and match up with the pairs in the current map container value.
					FScriptMapHelper_InContainer MapValue(MapProp, InDataPtr, ArrayIdx);
					FScriptMapHelper_InContainer DefValue(MapProp, InDefaultDataPtr, ArrayIdx);
					for (int32 ValueIdx = 0; ValueIdx < MapValue.Num() && ValueIdx < DefValue.Num(); ++ValueIdx)
					{
						ValidateInstancedObjectProperty(InOuter, MapProp->ValueProp, MapValue.GetValuePtr(ValueIdx), DefValue.GetValuePtr(ValueIdx));
					}
				}
			}
		}

		static void ValidateDefaultSubobjects_Inner(UObject* InOuter, UStruct* InStruct, void* InDataPtr, const void* InDefaultDataPtr)
		{
			// Iterate over all reference properties, including those inherited from the parent class hierarchy.
			for (const FProperty* RefProp = InStruct->RefLink; RefProp; RefProp = RefProp->NextRef)
			{
				// We only need to consider 'Instanced' reference properties here (e.g. instanced subobjects created at construction time).
				if (RefProp->ContainsInstancedObjectProperty())
				{
					ValidateInstancedObjectProperty(InOuter, RefProp, InDataPtr, InDefaultDataPtr);
				}
			}
		}
		
	public:
		/**
		 * Utility method that iterates the reference property chain for a given object's underlying (non-native) type, and validates all
		 * references to any instanced default subobjects (DSOs), or subobjects that are instanced natively when we first construct and
		 * initialize the object through its native super class hierarchy. Current validation steps include:
		 *
		 *	a) Ensure that the given object has a non-NULL value for each reference property inherited from its native super chain, when
		 *	   compared to its closest native parent class default object (CDO). Blueprints cannot mark default subobjects as "optional,"
		 *	   which means that if the closest native parent CDO has a non-NULL value for a property that's referencing a default subobject
		 *	   instance, then we expect that the Blueprint's CDO should also have a non-NULL value for the same property, but referencing
		 *	   its own unique instance of a subobject with the same name. In some cases (due to reinstancing bugs perhaps), these references
		 *	   can unintentionally be serialized as NULL to the Blueprint asset, which can lead to data loss and corrupted assets that are
		 *	   otherwise unrecoverable. This both emits a warning to the log when validation fails, as well as attempts to restore these
		 *	   references back to the unique instance that was constructed/initialized for the object, in order to allow for data recovery.
		 *
		 * Any additional validation steps that are implemented as part of this path in the future should be noted above for completeness.
		 */
		static void ValidateDefaultSubobjects(UObject* InObject)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintGeneratedClassUtils::ValidateDefaultSubobjects);

			if (GBlueprintDefaultSubobjectValidationDisabled)
			{
				return;
			}

			check(InObject);

			UClass* ObjClass = InObject->GetClass();
			check(ObjClass);

			// No need to validate non-native class types.
			if (ObjClass->IsNative())
			{
				return;
			}

			// Find the closest native super class in the inheritance hierarchy.
			UClass* NativeParentClass = FindFirstNativeClassInHierarchy(ObjClass);
			check(NativeParentClass);

			// Grab a reference to its default object. We'll use it as the basis for comparison below.
			UObject* NativeParentCDO = NativeParentClass->GetDefaultObject(false);
			check(NativeParentCDO);

			// Validate this object's DSO member references against its closest native super class defaults.
			ValidateDefaultSubobjects_Inner(InObject, NativeParentClass, InObject, NativeParentCDO);
		}
#endif	// WITH_EDITOR
	};
}

UBlueprintGeneratedClass::UBlueprintGeneratedClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	, UberGraphFunctionKey(0)
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
{
	NumReplicatedProperties = 0;
	bHasCookedComponentInstancingData = false;
	bSupportsDynamicInstancedReference = false;
	bCustomPropertyListForPostConstructionInitialized = false;
#if WITH_EDITORONLY_DATA
	bIsInternalClass = false;
	bIsSparseClassDataSerializable = false;
#endif
}

void UBlueprintGeneratedClass::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Default__BlueprintGeneratedClass uses its own AddReferencedObjects function.
		CppClassStaticFunctions = UOBJECT_CPPCLASS_STATICFUNCTIONS_FORCLASS(UBlueprintGeneratedClass);
	}
}

void UBlueprintGeneratedClass::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
#if WITH_EDITOR
	// Make BPGC from a cooked package standalone so it doesn't get GCed
	if (GEditor && bCooked)
	{
		SetFlags(RF_Standalone);
	}
#endif //if WITH_EDITOR

	if (!bCooked)
	{
		if (GetAuthoritativeClass() != this)
		{
			return;
		}

		UObject* ClassCDO = GetDefaultObject(false);

		// Go through the CDO of the class, and make sure we don't have any legacy components that aren't instanced hanging on.
		struct FCheckIfComponentChildHelper
		{
			static bool IsComponentChild(UObject* CurrObj, const UObject* CDO)
			{
				UObject*  OuterObject = CurrObj ? CurrObj->GetOuter() : nullptr;
				const bool bValidOuter = OuterObject && (OuterObject != CDO);
				return bValidOuter ? (OuterObject->IsDefaultSubobject() || IsComponentChild(OuterObject, CDO)) : false;
			};
		};

		if (ClassCDO)
		{
			ForEachObjectWithOuter(ClassCDO, [ClassCDO](UObject* CurrObj)
			{
				const bool bComponentChild = FCheckIfComponentChildHelper::IsComponentChild(CurrObj, ClassCDO);
				if (!CurrObj->IsDefaultSubobject() && !CurrObj->IsRooted() && !bComponentChild)
				{
					CurrObj->MarkAsGarbage();
				}
			});
		}

		if (GetLinkerUEVersion() < VER_UE4_CLASS_NOTPLACEABLE_ADDED)
		{
			// Make sure the placeable flag is correct for all blueprints
			UBlueprint* Blueprint = Cast<UBlueprint>(ClassGeneratedBy);
			if (ensure(Blueprint) && Blueprint->BlueprintType != BPTYPE_MacroLibrary)
			{
				ClassFlags &= ~CLASS_NotPlaceable;
			}
		}

#if UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
		// Patch the fast calls (needed as we can't bump engine version to serialize it directly in UFunction right now)
		for (const FEventGraphFastCallPair& Pair : FastCallPairs_DEPRECATED)
		{
			Pair.FunctionToPatch->EventGraphFunction = UberGraphFunction;
			Pair.FunctionToPatch->EventGraphCallOffset = Pair.EventGraphCallOffset;
		}
#endif
	}
#endif // WITH_EDITORONLY_DATA

	// Update any component names that have been redirected
	if (!FPlatformProperties::RequiresCookedData() && GetAllowNativeComponentClassOverrides())
	{
		for (FBPComponentClassOverride& Override : ComponentClassOverrides)
		{
			const FString ComponentName = Override.ComponentName.ToString();
			UClass* ClassToCheck = this;
			while (ClassToCheck)
			{
				if (const TMap<FString, FString>* ValueChanges = FCoreRedirects::GetValueRedirects(ECoreRedirectFlags::Type_Class, ClassToCheck))
				{
					if (const FString* NewComponentName = ValueChanges->Find(ComponentName))
					{
						Override.ComponentName = **NewComponentName;
						break;
					}
				}
				ClassToCheck = ClassToCheck->GetSuperClass();
			}
		}
	}

	InitializeFieldNotifies();

	AssembleReferenceTokenStream(true);
}

void UBlueprintGeneratedClass::InitializeFieldNotifies()
{
	//Initialize the interface with the computed FieldNotifies
	FieldNotifiesStartBitNumber = 0;
	if (FieldNotifies.Num() && ImplementsInterface(UNotifyFieldValueChanged::StaticClass()) && ensure(GetDefaultObject(false)))
	{
		int32 NumberOfField = 0;
		TScriptInterface<INotifyFieldValueChanged>(GetDefaultObject(false))->GetFieldNotificationDescriptor().ForEachField(this, [&NumberOfField](::UE::FieldNotification::FFieldId FielId)
			{
				++NumberOfField;
				return true;
			});
		FieldNotifiesStartBitNumber = NumberOfField - FieldNotifies.Num();
		ensureMsgf(FieldNotifiesStartBitNumber >= 0, TEXT("The FieldNotifyStartIndex is negative. The number of field should be positive."));
	}
}

void UBlueprintGeneratedClass::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	FString NativeParentClassName;
	FString ParentClassName;

	if (UClass* ParentClass = GetSuperClass())
	{
		ParentClassName = FObjectPropertyBase::GetExportPath(ParentClass);

		// Walk up until we find a native class (ie 'while they are BP classes')
		UClass* NativeParentClass = ParentClass;
		while (Cast<UBlueprintGeneratedClass>(NativeParentClass))
		{
			NativeParentClass = NativeParentClass->GetSuperClass();
		}
		NativeParentClassName = FObjectPropertyBase::GetExportPath(NativeParentClass);
	}
	else
	{
		NativeParentClassName = ParentClassName = TEXT("None");
	}

	Context.AddTag(FAssetRegistryTag(FBlueprintTags::ParentClassPath, ParentClassName, FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::NativeParentClassPath, NativeParentClassName, FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag(FBlueprintTags::ClassFlags, FString::FromInt((uint32)GetClassFlags()), FAssetRegistryTag::TT_Hidden));

#if WITH_EDITORONLY_DATA
	// Get editor-only tags; on a cooked BPGC, those tags are deserialized into CookedEditorTags, otherwise generate them for the BP
	const FEditorTags* EditorTagsToAdd = &CookedEditorTags;
	FEditorTags EditorTags;
	if (CookedEditorTags.Num() == 0)
	{
		GetEditorTags(EditorTags);
		EditorTagsToAdd = &EditorTags;
	}

	for (const auto& EditorTag : *EditorTagsToAdd)
	{
		Context.AddTag(FAssetRegistryTag(EditorTag.Key, EditorTag.Value, FAssetRegistryTag::TT_Hidden));
	}

	if (const UObject* CDO = GetDefaultObject())
	{
		if (const IAssetRegistryTagProviderInterface* AssetRegistryProvider = Cast<IAssetRegistryTagProviderInterface>(CDO))
		{
			if (AssetRegistryProvider->ShouldAddCDOTagsToBlueprintClass())
			{
				CDO->GetAssetRegistryTags(Context);
			}
		}
	}
#endif //#if WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/*
	 * Can't use GetExtendedAssetRegistryTagsForSave here because:
	 *	- UBlueprint is an asset in editor builds only.
	 *	- UBlueprintGeneratedClass is an asset in cooked builds only.
	 *	- Extended tags are not present in cooked builds.
	 *
	 * See UBlueprint::GetExtendedAssetRegistryTagsForSave.
	 */
	if (UE::IsSavingPackage() && !IsRunningCookCommandlet())
	{
		if (AActor* BlueprintCDO = Cast<AActor>(GetDefaultObject(false)))
		{
			if (UPackage* BlueprintCDOPackage = BlueprintCDO->GetPackage())
			{
				if (!FPackageName::IsTempPackage(BlueprintCDOPackage->GetName()) && !BlueprintCDOPackage->HasAnyPackageFlags(PKG_PlayInEditor))
				{
					FWorldPartitionActorDescUtils::AppendAssetDataTagsFromActor(BlueprintCDO, Context);
				}
			}
		}
	}
#endif
}

#if WITH_EDITOR
void UBlueprintGeneratedClass::GetAdditionalAssetDataObjectsForCook(FArchiveCookContext& CookContext,
	TArray<UObject*>& OutObjects) const
{
	Super::GetAdditionalAssetDataObjectsForCook(CookContext, OutObjects);
	UBlueprint* Blueprint = Cast<UBlueprint>(ClassGeneratedBy);
	if (Blueprint && Blueprint->GetPackage() == GetPackage())
	{
		OutObjects.Add(Blueprint);
	}
}

void UBlueprintGeneratedClass::ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const
{
	Super::ThreadedPostLoadAssetRegistryTagsOverride(Context);

	auto FixTagValueShortClassName = [&Context](FName TagName, FAssetRegistryTag::ETagType TagType)
	{
		FString TagValue = Context.GetAssetData().GetTagValueRef<FString>(TagName);
		if (!TagValue.IsEmpty() && TagValue != TEXT("None"))
		{
			if (UClass::TryFixShortClassNameExportPath(TagValue, ELogVerbosity::Warning,
				TEXT("UBlueprintGeneratedClass::ThreadedPostLoadAssetRegistryTagsOverride"), true /* bClearOnError */))
			{
				Context.AddTagToUpdate(FAssetRegistryTag(TagName, TagValue, TagType));
			}
		}
	};

	FixTagValueShortClassName(FBlueprintTags::ParentClassPath, FAssetRegistryTag::TT_Alphabetical);
	FixTagValueShortClassName(FBlueprintTags::NativeParentClassPath, FAssetRegistryTag::TT_Alphabetical);
}
#endif // WITH_EDITOR

FPrimaryAssetId UBlueprintGeneratedClass::GetPrimaryAssetId() const
{
	FPrimaryAssetId AssetId;
	if (!GetDefaultObject(false))
	{
		// All UBlueprintGeneratedClass objects should have a pointer to their generated ClassDefaultObject, except for the
		// CDO itself of the UBlueprintGeneratedClass class.
		verify(HasAnyFlags(RF_ClassDefaultObject));
		return AssetId;
	}

	AssetId = GetDefaultObject(false)->GetPrimaryAssetId();

	/*
	if (!AssetId.IsValid())
	{ 
		FName AssetType = NAME_None; // TODO: Support blueprint-only primary assets with a class flag. No way to guess at type currently
		FName AssetName = FPackageName::GetShortFName(GetOutermost()->GetFName());
		return FPrimaryAssetId(AssetType, AssetName);
	}
	*/

	return AssetId;
}

#if WITH_EDITOR

UClass* UBlueprintGeneratedClass::GetAuthoritativeClass()
{
 	if (nullptr == ClassGeneratedBy)
 	{
		// If this is a cooked blueprint, the generatedby class will have been discarded so we'll just have to assume we're authoritative!
		if (bCooked || RootPackageHasAnyFlags(PKG_Cooked))
		{ 
			return this;
		}
		else
		{
			UE_LOGF(LogBlueprint, Fatal, "UBlueprintGeneratedClass::GetAuthoritativeClass: ClassGeneratedBy is null. class '%ls'", *GetPathName());
		}
 	}

	UBlueprint* GeneratingBP = CastChecked<UBlueprint>(ClassGeneratedBy);

	return (GeneratingBP->GeneratedClass != NULL) ? GeneratingBP->GeneratedClass : this;
}

void UBlueprintGeneratedClass::ConditionalRecompileClass(FUObjectSerializeContext* InLoadContext)
{
	FBlueprintCompilationManager::FlushCompilationQueue(InLoadContext);
}

void UBlueprintGeneratedClass::FlushCompilationQueueForLevel()
{
	if(Cast<ULevelScriptBlueprint>(ClassGeneratedBy))
	{
		FBlueprintCompilationManager::FlushCompilationQueue(nullptr);
	}
}

UObject* UBlueprintGeneratedClass::GetArchetypeForCDO() const
{
	if (OverridenArchetypeForCDO)
	{
		ensure(OverridenArchetypeForCDO->IsA(GetSuperClass()));
		return OverridenArchetypeForCDO;
	}

	return Super::GetArchetypeForCDO();
}
#endif //WITH_EDITOR

void UBlueprintGeneratedClass::SerializeDefaultObject(UObject* Object, FStructuredArchive::FSlot Slot)
{
	UE::TScopeLock SerializeAndPostLoadLock(SerializeAndPostLoadCritical);
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

#if WITH_EDITOR
	using namespace UE::Runtime::Engine::Private;
	if (UnderlyingArchive.IsSaving() && !UnderlyingArchive.IsObjectReferenceCollector() && Object == GetDefaultObject(false))
	{
		// Validate/fix up default subobjects prior to saving the CDO.
		FBlueprintGeneratedClassUtils::ValidateDefaultSubobjects(Object);
	}
#endif	// WITH_EDITOR

	Super::SerializeDefaultObject(Object, Slot);

	if (UnderlyingArchive.IsLoading() && !UnderlyingArchive.IsObjectReferenceCollector() && Object == GetDefaultObject(false))
	{
#if WITH_EDITOR
		// Validate/fix up default subobjects after serializing the CDO (e.g. fix up any unexpected NULL refs).
		FBlueprintGeneratedClassUtils::ValidateDefaultSubobjects(Object);
#endif	// WITH_EDITOR

		CreatePersistentUberGraphFrame(Object, true);

		// On load, build the custom property list used in post-construct initialization logic. Note that in the editor, this will be refreshed during compile-on-load.
		// @TODO - Potentially make this serializable (or cooked data) to eliminate the slight load time cost we'll incur below to generate this list in a cooked build. For now, it's not serialized since the raw FProperty references cannot be saved out.
		UpdateCustomPropertyListForPostConstruction();

		const FString BPGCName = GetName();
		auto BuildCachedPropertyDataLambda = [BPGCName](FBlueprintCookedComponentInstancingData& CookedData, UActorComponent* SourceTemplate, FString CompVarName)
		{
			if (CookedData.bHasValidCookedData)
			{
				// This feature requires EDL at cook time, so ensure that the source template is also fully loaded at this point.
				// Also ensure that the source template is not a class default object; it must always be a unique archetype object.
				if (SourceTemplate != nullptr
					&& ensure(!SourceTemplate->HasAnyFlags(RF_NeedLoad))
					&& ensure(!SourceTemplate->HasAnyFlags(RF_ClassDefaultObject)))
				{
					CookedData.BuildCachedPropertyDataFromTemplate(SourceTemplate);
				}
				else
				{
					// This situation is unexpected; templates that are filtered out by context should not be generating fast path data at cook time. Emit a warning about this.
					UE_LOGF(LogBlueprint, Warning, "BPComp fast path (%ls.%ls) : Invalid source template. Will use slow path for dynamic instancing.", *BPGCName, *CompVarName);

					// Invalidate the cooked data so that we fall back to using the slow path when dynamically instancing this node.
					CookedData.bHasValidCookedData = false;
				}
			}
		};

#if WITH_EDITOR
		const bool bShouldUseCookedComponentInstancingData = bHasCookedComponentInstancingData && !GIsEditor;
#else
		const bool bShouldUseCookedComponentInstancingData = bHasCookedComponentInstancingData;
#endif
		// Generate "fast path" instancing data for inherited SCS node templates.
		if (InheritableComponentHandler && bShouldUseCookedComponentInstancingData)
		{
			for (auto RecordIt = InheritableComponentHandler->CreateRecordIterator(); RecordIt; ++RecordIt)
			{
				BuildCachedPropertyDataLambda(RecordIt->CookedComponentInstancingData, RecordIt->ComponentTemplate, RecordIt->ComponentKey.GetSCSVariableName().ToString());
			}
		}

		if (bShouldUseCookedComponentInstancingData)
		{
			// Generate "fast path" instancing data for SCS node templates owned by this Blueprint class.
			if (SimpleConstructionScript)
			{
				const TArray<USCS_Node*>& AllSCSNodes = SimpleConstructionScript->GetAllNodes();
				for (USCS_Node* SCSNode : AllSCSNodes)
				{
					BuildCachedPropertyDataLambda(SCSNode->CookedComponentInstancingData, SCSNode->ComponentTemplate, SCSNode->GetVariableName().ToString());
				}
			}

			// Generate "fast path" instancing data for UCS/AddComponent node templates.
			if (CookedComponentInstancingData.Num() > 0)
			{
				for (UActorComponent* ComponentTemplate : ComponentTemplates)
				{
					if (ComponentTemplate)
					{
						FBlueprintCookedComponentInstancingData* ComponentInstancingData = CookedComponentInstancingData.Find(ComponentTemplate->GetFName());
						if (ComponentInstancingData != nullptr)
						{
							BuildCachedPropertyDataLambda(*ComponentInstancingData, ComponentTemplate, ComponentTemplate->GetName());
						}
					}
				}
			}
		}
	}

	UnderlyingArchive.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	bool bSkipSparseClassDataSerialization = false;
	if(UnderlyingArchive.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SparseClassDataStructSerialization)
	{
		if (UnderlyingArchive.IsSaving())
		{
			UScriptStruct* SerializedSparseClassDataStruct = SparseClassDataStruct;
			if (!UE::Reflection::DoesSparseClassDataOverrideArchetype(this, [&UnderlyingArchive](FProperty* P) { return P->ShouldSerializeValue(UnderlyingArchive);}))
			{
				// If this class doesn't override the sparse class data of its archetype, then we can skip saving it 
				// since it can be lazily regenerated from the archetype data on load
				SerializedSparseClassDataStruct = nullptr;
				bSkipSparseClassDataSerialization = true;
			}
			UnderlyingArchive << SerializedSparseClassDataStruct;
		}
		else if (UnderlyingArchive.IsLoading())
		{
			UScriptStruct* SerializedSparseClassDataStruct = nullptr;
			UnderlyingArchive << SerializedSparseClassDataStruct;
			if (SparseClassDataStruct != SerializedSparseClassDataStruct)
			{
				CleanupSparseClassData();
				SparseClassDataStruct = SerializedSparseClassDataStruct;
			}
			if (!SparseClassDataStruct)
			{
				// Missing or failed to load sparse class data struct - possible that the parent class was deleted, or regenerated on load
				// so seek past where this CDO was serialized as we cannot read the serialized sparse class data any more.
				// Note this happens in majority of cases with classes that have no sparse class data attached too. In that case 
				// this 'skip' over the remaining part of the archive should have no ill effects as the seek should effectively be zero
				bSkipSparseClassDataSerialization = true;
			}
		}
	}

#if WITH_EDITORONLY_DATA
	if (bIsSparseClassDataSerializable || GetPackage()->HasAnyPackageFlags(PKG_Cooked))
#endif
	{
		if(bSkipSparseClassDataSerialization)
		{
			// Seek to after sparse class data rather than load it.
			if (UnderlyingArchive.IsLoading())
			{
				if (const FLinkerLoad* Linker = Object->GetLinker())
				{
					const int32 LinkerIndex = Object->GetLinkerIndex();
					const FObjectExport& Export = Linker->ExportMap[LinkerIndex];
					UnderlyingArchive.Seek(Export.SerialOffset + Export.SerialSize);
				}
			}
		}
		else if (SparseClassDataStruct)
		{
			SerializeSparseClassData(FStructuredArchiveFromArchive(UnderlyingArchive).GetSlot());
		}
	}

	if (UnderlyingArchive.IsLoading())
	{
		if (!SparseClassDataStruct)
		{
			SparseClassDataStruct = GetSparseClassDataArchetypeStruct();
		}

#if WITH_EDITOR
		if (!GetPackage()->HasAnyPackageFlags(PKG_Cooked))
		{
			PrepareToConformSparseClassData(GetSparseClassDataArchetypeStruct());
			if(UObject* CDO = GetDefaultObject(false))
			{
				// conform immediately, so that child types can correct delta serialize their
				// sparse data - we can't check RF_LoadCompleted because it is going to be set by the 
				// preload call that is causing SerializeDefaultObject to run, but Super::SerializeDefaultObject
				// has run...
				ConformSparseClassData(CDO);
				CDO->MoveDataToSparseClassDataStruct();
			}
		}
#endif
	}
}

void UBlueprintGeneratedClass::PostLoadDefaultObject(UObject* Object)
{
	UE::TScopeLock SerializeAndPostLoadLock(SerializeAndPostLoadCritical);

	Super::PostLoadDefaultObject(Object);

	if (Object == GetDefaultObject(false))
	{
		// Rebuild the custom property list used in post-construct initialization logic. Note that PostLoad() may have altered some serialized properties.
		UpdateCustomPropertyListForPostConstruction();

		// Restore any property values from config file
		if (HasAnyClassFlags(CLASS_Config))
		{
			GetDefaultObject(false)->LoadConfig();
		}
	}

#if WITH_EDITOR
	if (!GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		ConformSparseClassData(Object);
		Object->MoveDataToSparseClassDataStruct();
	}

	if (Object->GetSparseClassDataStruct())
	{
		// now that any data has been moved into the sparse data structure we can safely serialize it
		bIsSparseClassDataSerializable = true;
	}
#endif
}

#if WITH_EDITOR
void UBlueprintGeneratedClass::ReinitializeObjectAfterCompile(UObject* Object) const
{
	HandleReinitializeObjectAfterCompile(Object);
}

void UBlueprintGeneratedClass::PrepareToConformSparseClassData(UScriptStruct* SparseClassDataArchetypeStruct)
{
	checkf(SparseClassDataPendingConformStruct.IsExplicitlyNull() && SparseClassDataPendingConform == nullptr, TEXT("PrepareToConformSparseClassData was called while data was already pending conform!"));

	if (SparseClassDataStruct)
	{
		if (SparseClassDataStruct != SparseClassDataArchetypeStruct)
		{
			if (SparseClassDataArchetypeStruct)
			{
				SparseClassDataPendingConformStruct = SparseClassDataStruct;
				SparseClassDataPendingConform = SparseClassData;

				SparseClassDataStruct = SparseClassDataArchetypeStruct;
				SparseClassData = nullptr;
			}
			else
			{
				CleanupSparseClassData();
				SparseClassDataStruct = nullptr;
			}
		}
	}
}

void UBlueprintGeneratedClass::ConformSparseClassData(UObject* Object)
{
	if (UScriptStruct* SparseClassDataPendingConformStructPtr = SparseClassDataPendingConformStruct.Get();
		SparseClassDataPendingConformStructPtr && SparseClassDataPendingConform)
	{
		// Always allow the CDO first refusal at handling the conversion
		if (!Object->ConformSparseClassDataStruct(SparseClassDataPendingConformStructPtr, SparseClassDataPendingConform))
		{
			UScriptStruct* SparseClassDataArchetypeStruct = GetSparseClassDataArchetypeStruct();

			// Copy common properties if the structs are related types
			UScriptStruct* SparseClassDataStructToCopy = nullptr;
			if (SparseClassDataArchetypeStruct->IsChildOf(SparseClassDataPendingConformStructPtr))
			{
				SparseClassDataStructToCopy = SparseClassDataPendingConformStructPtr;
			}
			else if (SparseClassDataPendingConformStructPtr->IsChildOf(SparseClassDataArchetypeStruct))
			{
				SparseClassDataStructToCopy = SparseClassDataArchetypeStruct;
			}
			if (SparseClassDataStructToCopy)
			{
				// Copy all properties from SparseClassDataPendingConform into the current sparse class data struct.
				// NOTE: Avoids the copy assignment operators of the class properties, as they may not preserve all data.
				void* const ThisSparseClassData = GetOrCreateSparseClassData();
				for (TFieldIterator<FProperty> It(SparseClassDataStructToCopy); It; ++It)
				{
					It->CopyCompleteValue_InContainer(ThisSparseClassData, SparseClassDataPendingConform);
				}
			}
		}

		SparseClassDataPendingConformStructPtr->DestroyStruct(SparseClassDataPendingConform);
	}

	SparseClassDataPendingConformStruct = nullptr;
	if (SparseClassDataPendingConform)
	{
		FMemory::Free(SparseClassDataPendingConform);
		SparseClassDataPendingConform = nullptr;
	}
}
#endif

bool UBlueprintGeneratedClass::BuildCustomPropertyListForPostConstruction(FCustomPropertyListNode*& InPropertyList, UStruct* InStruct, const uint8* DataPtr, const uint8* DefaultDataPtr)
{
	using namespace UE::Runtime::Engine::Private;

	const UClass* OwnerClass = Cast<UClass>(InStruct);
	FCustomPropertyListNode** CurrentNodePtr = &InPropertyList;

	for (FProperty* Property = InStruct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (FBlueprintGeneratedClassUtils::ShouldInitializePropertyDuringPostConstruction(*Property))
		{
			// Some properties require a full value comparison; check for those cases here.
			const bool bAlwaysUseCompleteValue = FBlueprintGeneratedClassUtils::RequiresCompleteValueForPostConstruction(Property);

			for (int32 Idx = 0; Idx < Property->ArrayDim; Idx++)
			{
				const uint8* PropertyValue = Property->ContainerPtrToValuePtr<uint8>(DataPtr, Idx);
				const uint8* DefaultPropertyValue = Property->ContainerPtrToValuePtrForDefaults<uint8>(InStruct, DefaultDataPtr, Idx);

				bool bUseCompleteValue = bAlwaysUseCompleteValue;
				if (!bUseCompleteValue)
				{
					// If this is a struct property, recurse to pull out any fields that differ from the native CDO.
					if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
					{
						// Create a new node for the struct property.
						*CurrentNodePtr = new FCustomPropertyListNode(Property, Idx);
						CustomPropertyListForPostConstruction.Add(*CurrentNodePtr);

						UScriptStruct::ICppStructOps* CppStructOps = nullptr;
						if (StructProperty->Struct)
						{
							CppStructOps = StructProperty->Struct->GetCppStructOps();
						}

						// Check if we should initialize using the full value (e.g. a USTRUCT with one or more non-reflected fields).
						bool bIsIdentical = false;
						const uint32 PortFlags = 0;
						if (!CppStructOps || !CppStructOps->HasIdentical() || !CppStructOps->Identical(PropertyValue, DefaultPropertyValue, PortFlags, bIsIdentical))
						{
							// Recursively gather up all struct fields that differ and assign to the current node's sub property list.
							bIsIdentical = !BuildCustomPropertyListForPostConstruction((*CurrentNodePtr)->SubPropertyList, StructProperty->Struct, PropertyValue, DefaultPropertyValue);
						}

						if (!bIsIdentical)
						{
							// Advance to the next node in the list.
							CurrentNodePtr = &(*CurrentNodePtr)->PropertyListNext;
						}
						else
						{
							// Remove the node for the struct property since it does not differ from the native CDO.
							CustomPropertyListForPostConstruction.RemoveAt(CustomPropertyListForPostConstruction.Num() - 1);

							// Clear the current node ptr since the array will have freed up the memory it referenced.
							*CurrentNodePtr = nullptr;
						}
					}
					else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
					{
						// Create a new node for the array property.
						*CurrentNodePtr = new FCustomPropertyListNode(Property, Idx);
						CustomPropertyListForPostConstruction.Add(*CurrentNodePtr);

						// Recursively gather up all array item indices that differ and assign to the current node's sub property list.
						if (BuildCustomArrayPropertyListForPostConstruction(ArrayProperty, (*CurrentNodePtr)->SubPropertyList, PropertyValue, DefaultPropertyValue))
						{
							// Advance to the next node in the list.
							CurrentNodePtr = &(*CurrentNodePtr)->PropertyListNext;
						}
						else
						{
							// Remove the node for the array property since it does not differ from the native CDO.
							CustomPropertyListForPostConstruction.RemoveAt(CustomPropertyListForPostConstruction.Num() - 1);

							// Clear the current node ptr since the array will have freed up the memory it referenced.
							*CurrentNodePtr = nullptr;
						}
					}
					else
					{
						// Not explicitly handled above; fall back to using a full value comparison and emit this property if anything differs.
						bUseCompleteValue = true;
					}
				}
				
				if (bUseCompleteValue && !Property->Identical(PropertyValue, DefaultPropertyValue))
				{
					// Create a new node, link it into the chain and add it into the array.
					*CurrentNodePtr = new FCustomPropertyListNode(Property, Idx);
					CustomPropertyListForPostConstruction.Add(*CurrentNodePtr);

					// Advance to the next node ptr.
					CurrentNodePtr = &(*CurrentNodePtr)->PropertyListNext;
				}
			}
		}
	}

	// This will be non-NULL if the above found at least one property value that differs from the native CDO.
	return (InPropertyList != nullptr);
}

bool UBlueprintGeneratedClass::BuildCustomArrayPropertyListForPostConstruction(FArrayProperty* ArrayProperty, FCustomPropertyListNode*& InPropertyList, const uint8* DataPtr, const uint8* DefaultDataPtr, int32 StartIndex)
{
	FCustomPropertyListNode** CurrentArrayNodePtr = &InPropertyList;

	FScriptArrayHelper ArrayValueHelper(ArrayProperty, DataPtr);
	FScriptArrayHelper DefaultArrayValueHelper(ArrayProperty, DefaultDataPtr);

	for (int32 ArrayValueIndex = StartIndex; ArrayValueIndex < ArrayValueHelper.Num(); ++ArrayValueIndex)
	{
		const int32 DefaultArrayValueIndex = ArrayValueIndex - StartIndex;
		if (DefaultArrayValueIndex < DefaultArrayValueHelper.Num())
		{
			const uint8* ArrayPropertyValue = ArrayValueHelper.GetRawPtr(ArrayValueIndex);
			const uint8* DefaultArrayPropertyValue = DefaultArrayValueHelper.GetRawPtr(DefaultArrayValueIndex);

			if (FStructProperty* InnerStructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
			{
				// Create a new node for the item value at this index.
				*CurrentArrayNodePtr = new FCustomPropertyListNode(ArrayProperty, ArrayValueIndex);
				CustomPropertyListForPostConstruction.Add(*CurrentArrayNodePtr);

				UScriptStruct::ICppStructOps* CppStructOps = nullptr;
				if (InnerStructProperty->Struct)
				{
					CppStructOps = InnerStructProperty->Struct->GetCppStructOps();
				}

				// Check if we should initialize using the full value (e.g. a USTRUCT with one or more non-reflected fields).
				bool bIsIdentical = false;
				const uint32 PortFlags = 0;
				if (!CppStructOps || !CppStructOps->HasIdentical() || !CppStructOps->Identical(ArrayPropertyValue, DefaultArrayPropertyValue, PortFlags, bIsIdentical))
				{
					// Recursively gather up all struct fields that differ and assign to the array item value node's sub property list.
					bIsIdentical = !BuildCustomPropertyListForPostConstruction((*CurrentArrayNodePtr)->SubPropertyList, InnerStructProperty->Struct, ArrayPropertyValue, DefaultArrayPropertyValue);
				}

				if (!bIsIdentical)
				{
					// Advance to the next node in the list.
					CurrentArrayNodePtr = &(*CurrentArrayNodePtr)->PropertyListNext;
				}
				else
				{
					// Remove the node for the struct property since it does not differ from the native CDO.
					CustomPropertyListForPostConstruction.RemoveAt(CustomPropertyListForPostConstruction.Num() - 1);

					// Clear the current array item node ptr
					*CurrentArrayNodePtr = nullptr;
				}
			}
			else if (FArrayProperty* InnerArrayProperty = CastField<FArrayProperty>(ArrayProperty->Inner))
			{
				// Create a new node for the item value at this index.
				*CurrentArrayNodePtr = new FCustomPropertyListNode(ArrayProperty, ArrayValueIndex);
				CustomPropertyListForPostConstruction.Add(*CurrentArrayNodePtr);

				// Recursively gather up all array item indices that differ and assign to the array item value node's sub property list.
				if (BuildCustomArrayPropertyListForPostConstruction(InnerArrayProperty, (*CurrentArrayNodePtr)->SubPropertyList, ArrayPropertyValue, DefaultArrayPropertyValue))
				{
					// Advance to the next node in the list.
					CurrentArrayNodePtr = &(*CurrentArrayNodePtr)->PropertyListNext;
				}
				else
				{
					// Remove the node for the array property since it does not differ from the native CDO.
					CustomPropertyListForPostConstruction.RemoveAt(CustomPropertyListForPostConstruction.Num() - 1);

					// Clear the current array item node ptr
					*CurrentArrayNodePtr = nullptr;
				}
			}
			else if (!ArrayProperty->Inner->Identical(ArrayPropertyValue, DefaultArrayPropertyValue))
			{
				// Create a new node, link it into the chain and add it into the array.
				*CurrentArrayNodePtr = new FCustomPropertyListNode(ArrayProperty, ArrayValueIndex);
				CustomPropertyListForPostConstruction.Add(*CurrentArrayNodePtr);

				// Advance to the next array item node ptr.
				CurrentArrayNodePtr = &(*CurrentArrayNodePtr)->PropertyListNext;
			}
		}
		else
		{
			// NULL signals the end of the array value change at the current index.
			*CurrentArrayNodePtr = new FCustomPropertyListNode(nullptr, ArrayValueIndex);
			CustomPropertyListForPostConstruction.Add(*CurrentArrayNodePtr);

			// Don't need to record anything else.
			break;
		}
	}

	// Return true if the above found at least one array element that differs from the native CDO, or otherwise if the array sizes are different.
	return (InPropertyList != nullptr || ArrayValueHelper.Num() != DefaultArrayValueHelper.Num());
}

void UBlueprintGeneratedClass::UpdateCustomPropertyListForPostConstruction()
{
	using namespace UE::Runtime::Engine::Private;

	// Empty the current list.
	CustomPropertyListForPostConstruction.Reset();
	bCustomPropertyListForPostConstructionInitialized = false;

	// Find the first native antecedent. All non-native decendant properties are attached to the PostConstructLink chain (see UStruct::Link), so we only need to worry about properties owned by native super classes here.
	if (UClass* SuperClass = FBlueprintGeneratedClassUtils::FindFirstNativeClassInHierarchy(GetSuperClass()))
	{
		check(GetDefaultObject(false) != nullptr);

		// Recursively gather native class-owned property values that differ from defaults.
		FCustomPropertyListNode* PropertyList = nullptr;
		BuildCustomPropertyListForPostConstruction(PropertyList, SuperClass, (uint8*)GetDefaultObject(false), (uint8*)SuperClass->GetDefaultObject(false));
	}

	bCustomPropertyListForPostConstructionInitialized = true;
}

void UBlueprintGeneratedClass::SetupObjectInitializer(FObjectInitializer& ObjectInitializer) const
{
	for (const FBPComponentClassOverride& Override : ComponentClassOverrides)
	{
		ObjectInitializer.SetDefaultSubobjectClass(Override.ComponentName, Override.ComponentClass);
	}

	GetSuperClass()->SetupObjectInitializer(ObjectInitializer);
}

void UBlueprintGeneratedClass::PostInitInstance(UObject* InObj, FObjectInstancingGraph* InstanceGraph)
{
	GetSuperClass()->PostInitInstance(InObj, InstanceGraph);
}

void UBlueprintGeneratedClass::PostLoadInstance(UObject* InObj)
{
	GetSuperClass()->PostLoadInstance(InObj);
}

void UBlueprintGeneratedClass::InitPropertiesFromCustomList(uint8* DataPtr, const uint8* DefaultDataPtr)
{
	UE::TScopeLock SerializeAndPostLoadLock(SerializeAndPostLoadCritical);

	if (GBlueprintNativePropertyInitFastPathDisabled
		|| !ensureMsgf(bCustomPropertyListForPostConstructionInitialized, TEXT("Custom Property List Not Initialized for %s"), *GetPathNameSafe(this))) // Something went wrong, probably a race condition
	{
		// Slow path - initialize all inherited native properties, regardless of whether they have been modified away from natively-initialized values.
		using namespace UE::Runtime::Engine::Private;
		if (const UClass* NativeParentClass = FBlueprintGeneratedClassUtils::FindFirstNativeClassInHierarchy(GetSuperClass()))
		{
			for (TFieldIterator<FProperty> It(NativeParentClass); It; ++It)
			{
				const FProperty* Property = *It;
				check(Property);

				if (FBlueprintGeneratedClassUtils::ShouldInitializePropertyDuringPostConstruction(*Property))
				{
					Property->CopyCompleteValue_InContainer(DataPtr, DefaultDataPtr);
				}
			}
		}
	}
	else
	{
		// Note: It is valid to have a NULL custom property list when the 'initialized' flag is also set - this
		// implies that no inherited native properties have been modified and thus do not need to be initialized.
		if (const FCustomPropertyListNode* CustomPropertyList = GetCustomPropertyListForPostConstruction())
		{
			// Fast path - will initialize only the subset of inherited native properties that have been modified. The rest
			// have already been initialized by the native class ctor, and initializing them again here would be redundant.
			InitPropertiesFromCustomList(CustomPropertyList, this, DataPtr, DefaultDataPtr);
		}
	}
}

void UBlueprintGeneratedClass::InitPropertiesFromCustomList(const FCustomPropertyListNode* InPropertyList, UStruct* InStruct, uint8* DataPtr, const uint8* DefaultDataPtr)
{
	for (const FCustomPropertyListNode* CustomPropertyListNode = InPropertyList; CustomPropertyListNode; CustomPropertyListNode = CustomPropertyListNode->PropertyListNext)
	{
		uint8* PropertyValue = CustomPropertyListNode->Property->ContainerPtrToValuePtr<uint8>(DataPtr, CustomPropertyListNode->ArrayIndex);
		const uint8* DefaultPropertyValue = CustomPropertyListNode->Property->ContainerPtrToValuePtr<uint8>(DefaultDataPtr, CustomPropertyListNode->ArrayIndex);

		if (!InitPropertyFromSubPropertyList(CustomPropertyListNode->Property, CustomPropertyListNode->SubPropertyList, PropertyValue, DefaultPropertyValue))
		{
			// Unable to init properties from sub custom property list, fall back to the default copy value behavior
			CustomPropertyListNode->Property->CopySingleValue(PropertyValue, DefaultPropertyValue);
		}
	}
}

void UBlueprintGeneratedClass::InitArrayPropertyFromCustomList(const FArrayProperty* ArrayProperty, const FCustomPropertyListNode* InPropertyList, uint8* DataPtr, const uint8* DefaultDataPtr)
{
	FScriptArrayHelper DstArrayValueHelper(ArrayProperty, DataPtr);
	FScriptArrayHelper SrcArrayValueHelper(ArrayProperty, DefaultDataPtr);

	const int32 SrcNum = SrcArrayValueHelper.Num();
	const int32 DstNum = DstArrayValueHelper.Num();

	if (SrcNum > DstNum)
	{
		DstArrayValueHelper.AddValues(SrcNum - DstNum);
	}
	else if (SrcNum < DstNum)
	{
		DstArrayValueHelper.RemoveValues(SrcNum, DstNum - SrcNum);
	}

	int32 ArrayIndex = 0;
	for (const FCustomPropertyListNode* CustomArrayPropertyListNode = InPropertyList; CustomArrayPropertyListNode; CustomArrayPropertyListNode = CustomArrayPropertyListNode->PropertyListNext)
	{
		ArrayIndex = CustomArrayPropertyListNode->ArrayIndex;
		if (CustomArrayPropertyListNode->Property == nullptr)
		{
			// This signals the end of the default value change.
			break;
		}

		uint8* DstArrayItemValue = DstArrayValueHelper.GetRawPtr(ArrayIndex);
		const uint8* SrcArrayItemValue = SrcArrayValueHelper.GetRawPtr(ArrayIndex);

		if (DstArrayItemValue == nullptr && SrcArrayItemValue == nullptr)
		{
			continue;
		}

		if (!SrcArrayValueHelper.IsValidIndex(ArrayIndex)) // dst bounds were conformed above, so just need to check source
		{
			ensureMsgf(false,
				TEXT("InitArrayPropertyFromCustomList attempted out of bounds access within %s,"
				"this indicates a template was mutated without calling UpdateCustomPropertyListForPostConstruction"),
				*ArrayProperty->GetOwner<UStruct>()->GetFullName()
				);
			continue;
		}

		if (!InitPropertyFromSubPropertyList(ArrayProperty->Inner, CustomArrayPropertyListNode->SubPropertyList, DstArrayItemValue, SrcArrayItemValue))
		{
			// Unable to init properties from sub custom property list, fall back to the default copy value behavior
			ArrayProperty->Inner->CopyCompleteValue(DstArrayItemValue, SrcArrayItemValue);
		}
	}

	// If necessary, copy the remainder of the source array value to the destination. It's possible for the number of elements in the
	// derived Blueprint class default object to exceed the number of elements in the first antecedent native superclass default object,
	// since the custom property list that's generated for the array will cover only the subset of elements that are common to both sides.
	if (ArrayIndex < SrcNum)
	{
		if (ArrayProperty->Inner->HasAnyPropertyFlags(CPF_IsPlainOldData))
		{
			uint8* DstArrayItemValue = DstArrayValueHelper.GetRawPtr(ArrayIndex);
			const uint8* SrcArrayItemValue = SrcArrayValueHelper.GetRawPtr(ArrayIndex);

			FMemory::Memcpy(DstArrayItemValue, SrcArrayItemValue, (SrcNum - ArrayIndex) * ArrayProperty->Inner->GetElementSize());
		}
		else
		{
			for (; ArrayIndex < SrcNum; ++ArrayIndex)
			{
				uint8* DstArrayItemValue = DstArrayValueHelper.GetRawPtr(ArrayIndex);
				const uint8* SrcArrayItemValue = SrcArrayValueHelper.GetRawPtr(ArrayIndex);

				if (DstArrayItemValue == nullptr && SrcArrayItemValue == nullptr)
				{
					continue;
				}

				ArrayProperty->Inner->CopyCompleteValue(DstArrayItemValue, SrcArrayItemValue);
			}
		}
	}
}

bool UBlueprintGeneratedClass::InitPropertyFromSubPropertyList(const FProperty* Property, const FCustomPropertyListNode* SubPropertyList, uint8* PropertyValue, const uint8* DefaultPropertyValue)
{
	if (SubPropertyList != nullptr)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			InitPropertiesFromCustomList(SubPropertyList, StructProperty->Struct, PropertyValue, DefaultPropertyValue);
			return true;
		}
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			InitArrayPropertyFromCustomList(ArrayProperty, SubPropertyList, PropertyValue, DefaultPropertyValue);
			return true;
		}
		// No need to handle Sets and Maps as they are not optimized through the custom property list
	}
	return false;
}

bool UBlueprintGeneratedClass::IsFunctionImplementedInScript(FName InFunctionName) const
{
	UFunction* Function = FindFunctionByName(InFunctionName);
	return Function && Function->GetOuter() && Function->GetOuter()->IsA(UBlueprintGeneratedClass::StaticClass());
}

UInheritableComponentHandler* UBlueprintGeneratedClass::GetInheritableComponentHandler(const bool bCreateIfNecessary)
{
	static const FBoolConfigValueHelper EnableInheritableComponents(TEXT("Kismet"), TEXT("bEnableInheritableComponents"), GEngineIni);
	if (!EnableInheritableComponents)
	{
		return nullptr;
	}

	if (!InheritableComponentHandler && bCreateIfNecessary)
	{
		InheritableComponentHandler = NewObject<UInheritableComponentHandler>(this, FName("InheritableComponentHandler"));
	}

	return InheritableComponentHandler;
}


UObject* UBlueprintGeneratedClass::FindArchetype(const UClass* ArchetypeClass, const FName ArchetypeName) const
{
	UObject* Archetype = nullptr;

	// There are some rogue LevelScriptActors that still have a SimpleConstructionScript
	// and since preloading the SCS of a script in a world package is bad news, we need to filter them out
	if (SimpleConstructionScript && !IsChildOf<ALevelScriptActor>())
	{
#if WITH_EDITORONLY_DATA
		// On load, we may fix up AddComponent node templates to conform to the newer archetype naming convention. In that case, we use a map to find
		// the new template name in order to redirect to the appropriate archetype.
		const UBlueprint* Blueprint = Cast<const UBlueprint>(ClassGeneratedBy);
		const FName NewArchetypeName = Blueprint ? Blueprint->OldToNewComponentTemplateNames.FindRef(ArchetypeName) : NAME_None;
#endif
		// Component templates (archetypes) differ from the component class default object, and they are considered to be "default subobjects" owned
		// by the Blueprint Class instance. Also, unlike "default subobjects" on the native C++ side, component templates are not currently owned by the
		// Blueprint Class default object. Instead, they are owned by the Blueprint Class itself. And, just as native C++ default subobjects serve as the
		// "archetype" object for components instanced and outered to a native Actor class instance at construction time, Blueprint Component templates
		// also serve as the "archetype" object for components instanced and outered to a Blueprint Class instance at construction time. However, since
		// Blueprint Component templates are not owned by the Blueprint Class default object, we must search for them by name within the Blueprint Class.
		//
		// Native component subobjects are instanced using the same name as the default subobject (archetype). Thus, it's easy to find the archetype -
		// we just look for an object with the same name that's owned by (i.e. outered to) the Actor class default object. This is the default logic
		// that we're overriding here.
		//
		// Blueprint (non-native) component templates are split between SCS (SimpleConstructionScript) and AddComponent nodes in Blueprint function
		// graphs (e.g. ConstructionScript). Both templates use a unique naming convention within the scope of the Blueprint Class, but at construction
		// time, we choose a unique name that differs from the archetype name for each component instance. We do this partially to support nativization,
		// in which we need to explicitly guard against recycling objects at allocation time. For SCS component instances, the name we choose matches the
		// "variable" name that's also user-facing. Thus, when we search for archetypes, we do so using the SCS variable name, and not the archetype name.
		// Conversely, for AddComponent node-spawned instances, we do not have a user-facing variable name, so instead we choose a unique name that
		// incorporates the archetype name, but we append an index as well. The index is needed to support multiple invocations of the same AddComponent
		// node in a function graph, which can occur when the AddComponent node is wired to a flow-control node such as a ForEach loop, for example. Thus,
		// we still look for the archetype by name, but we must first ensure that the instance name is converted to its "base" name by removing the index.
#if WITH_EDITORONLY_DATA
		const FName ArchetypeBaseName = (NewArchetypeName != NAME_None) ? NewArchetypeName : FName(ArchetypeName, 0);
#else
		const FName ArchetypeBaseName = FName(ArchetypeName, 0);
#endif
		UBlueprintGeneratedClass* Class = const_cast<UBlueprintGeneratedClass*>(this);
		while (Class)
		{
			USimpleConstructionScript* ClassSCS = Class->SimpleConstructionScript;
			USCS_Node* SCSNode = nullptr;
			if (ClassSCS)
			{
				if (ClassSCS->HasAnyFlags(RF_NeedLoad))
				{
					ClassSCS->PreloadChain();
				}

				// We keep the index name here rather than the base name, in order to avoid potential
				// collisions between an SCS variable name and an existing AddComponent node template.
				// This is because old AddComponent node templates were based on the class display name.
				SCSNode = ClassSCS->FindSCSNode(ArchetypeName);
			}

			if (SCSNode)
			{
				// Ensure that the stored template is of the same type as the serialized object. Since
				// we match these by name, this handles the case where the Blueprint class was updated
				// after having previously serialized an instanced into another package (e.g. map). In
				// that case, the Blueprint class might contain an SCS node with the same name as the
				// previously-serialized object, but it might also have been switched to a different type.
				if (SCSNode->ComponentTemplate)
				{
					if (SCSNode->ComponentTemplate->IsA(ArchetypeClass))
					{
						Archetype = SCSNode->ComponentTemplate;
					}

					// If the component class is in the process of being reinstanced, archetype lookup
					// will fail the class check because the SCS template will already have been replaced
					// when instances try to look up their archetype, so return the already reinstanced one
					else if (ArchetypeClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
					{
						const FString ArchetypeClassName = ArchetypeClass->GetName();
						if (ArchetypeClassName.StartsWith("REINST_") && FStringView(ArchetypeClassName).RightChop(7).StartsWith(SCSNode->ComponentTemplate->GetClass()->GetName()))
						{
							Archetype = SCSNode->ComponentTemplate;
						}
					}
				}
			}
			else if (UInheritableComponentHandler* ICH = Class->GetInheritableComponentHandler())
			{
				if (ICH->HasAnyFlags(RF_NeedLoad))
				{
					// We must preload here to avoid creating exports with an uninitialized archetype
					ICH->PreloadAll();
				}

				// This would find either an SCS component template override (for which the archetype
				// name will match the SCS variable name), or an old AddComponent node template override
				// (for which the archetype name will match the override record's component template name).
				FComponentKey ComponentKey = ICH->FindKey(ArchetypeName);
				if (!ComponentKey.IsValid() && ArchetypeName != ArchetypeBaseName)
				{
					// We didn't find either an SCS override or an old AddComponent template override,
					// so now we look for a match with the base name; this would apply to new AddComponent
					// node template overrides, which use the base name (non-index form).
					ComponentKey = ICH->FindKey(ArchetypeBaseName);

					// If we found a match with an SCS key instead, treat this as a collision and throw it
					// out, because it should have already been found in the first search. This could happen
					// if an old AddComponent node template's base name collides with an SCS variable name.
					if (ComponentKey.IsValid() && ComponentKey.IsSCSKey())
					{
						ComponentKey = FComponentKey();
					}
				}

				// Avoid searching for an invalid key.
				if (ComponentKey.IsValid())
				{
					Archetype = ICH->GetOverridenComponentTemplate(ComponentKey);

					// Exclude archetypes that were previously destroyed by ValidateTemplates to work around issues with REINST classes sharing templates
					if (Archetype && Archetype->HasAnyFlags(RF_Transient))
					{
						Archetype = nullptr;
					}
				}
			}

			if (Archetype == nullptr)
			{
				// We'll get here if we failed to find the archetype in either the SCS or the ICH. In that case,
				// we first check the base name case. If that fails, then we may be looking for something other
				// than an AddComponent template. In that case, we check for an object that shares the instance name.
				Archetype = static_cast<UObject*>(FindObjectWithOuter(Class, ArchetypeClass, ArchetypeBaseName));
				if (Archetype == nullptr && ArchetypeName != ArchetypeBaseName)
				{
					Archetype = static_cast<UObject*>(FindObjectWithOuter(Class, ArchetypeClass, ArchetypeName));
				}

				// Walk up the class hierarchy until we either find a match or hit a native class.
				Class = (Archetype ? nullptr : Cast<UBlueprintGeneratedClass>(Class->GetSuperClass()));
			}
			else
			{
				Class = nullptr;
			}
		}
		
		if(Archetype == nullptr)
		{
			// We failed to find an archetype by name - but the ICH mangles
			// the DefaultSceneRoot name *and* tries to cull unused templates
			// so do a relaxed compare to find the real default scene root:
			static const FName DefaultSceneRootSCS("DefaultSceneRoot_GEN_VARIABLE");
			static const FName ICHDefaultSceneRoot("ICH-DefaultSceneRoot_GEN_VARIABLE");
			if(ArchetypeName == DefaultSceneRootSCS || ArchetypeName == ICHDefaultSceneRoot)
			{
				Class = const_cast<UBlueprintGeneratedClass*>(this);
				while (Class && !Archetype)
				{
					// look for an SCS template:
					if(const USimpleConstructionScript* SCS = Class->SimpleConstructionScript)
					{
						if(const USCS_Node* Node = SCS->GetDefaultSceneRootNode())
						{
							// Assets may have orphaned default scene root nodes - only honor
							// them if they still have a spot in the root nodes list:
							if(SCS->GetRootNodes().Contains(Node))
							{
								Archetype = Node->ComponentTemplate;
							}
						}
					}

					Class = Cast<UBlueprintGeneratedClass>(Class->GetSuperClass());
				}
			}
		}
	}

	return Archetype;
}

UDynamicBlueprintBinding* UBlueprintGeneratedClass::GetDynamicBindingObject(const UClass* ThisClass, UClass* BindingClass)
{
	check(ThisClass);
	UDynamicBlueprintBinding* DynamicBlueprintBinding = nullptr;
	if (const UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(ThisClass))
	{
		for (UDynamicBlueprintBinding* DynamicBindingObject : BPGC->DynamicBindingObjects)
		{
			if (DynamicBindingObject && (DynamicBindingObject->GetClass() == BindingClass))
			{
				DynamicBlueprintBinding = DynamicBindingObject;
				break;
			}
		}
	}

	return DynamicBlueprintBinding;
}

void UBlueprintGeneratedClass::BindDynamicDelegates(const UClass* ThisClass, UObject* InInstance)
{
	check(ThisClass && InInstance);
	if (!InInstance->IsA(ThisClass))
	{
		UE_LOGF(LogBlueprint, Warning, "BindComponentDelegates: '%ls' is not an instance of '%ls'.", *InInstance->GetName(), *ThisClass->GetName());
		return;
	}

	if (const UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(ThisClass))
	{
		for (UDynamicBlueprintBinding* DynamicBindingObject : BPGC->DynamicBindingObjects)
		{
			if (ensure(DynamicBindingObject))
			{
				DynamicBindingObject->BindDynamicDelegates(InInstance);
			}
		}
	}

	if (UClass* TheSuperClass = ThisClass->GetSuperClass())
	{
		BindDynamicDelegates(TheSuperClass, InInstance);
	}
}

#if WITH_EDITOR
void UBlueprintGeneratedClass::UnbindDynamicDelegates(const UClass* ThisClass, UObject* InInstance)
{
	check(ThisClass && InInstance);
	if (!InInstance->IsA(ThisClass))
	{
		UE_LOGF(LogBlueprint, Warning, "UnbindDynamicDelegates: '%ls' is not an instance of '%ls'.", *InInstance->GetName(), *ThisClass->GetName());
		return;
	}

	if (const UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(ThisClass))
	{
		for (UDynamicBlueprintBinding* DynamicBindingObject : BPGC->DynamicBindingObjects)
		{
			if (ensure(DynamicBindingObject))
			{
				DynamicBindingObject->UnbindDynamicDelegates(InInstance);
			}
		}
	}

	if (UClass* TheSuperClass = ThisClass->GetSuperClass())
	{
		UnbindDynamicDelegates(TheSuperClass, InInstance);
	}
}

void UBlueprintGeneratedClass::UnbindDynamicDelegatesForProperty(UObject* InInstance, const FObjectProperty* InObjectProperty)
{
	for (int32 Index = 0; Index < DynamicBindingObjects.Num(); ++Index)
	{
		if ( ensure(DynamicBindingObjects[Index] != NULL) )
		{
			DynamicBindingObjects[Index]->UnbindDynamicDelegatesForProperty(InInstance, InObjectProperty);
		}
	}
}
#endif

bool UBlueprintGeneratedClass::GetGeneratedClassesHierarchy(const UClass* InClass, TArray<const UBlueprintGeneratedClass*>& OutBPGClasses)
{
	OutBPGClasses.Empty();

	return ForEachGeneratedClassInHierarchy(InClass, [&OutBPGClasses](const UBlueprintGeneratedClass* BPGClass)
	{
		OutBPGClasses.Add(BPGClass);
		return true;
	});
}

bool UBlueprintGeneratedClass::ForEachGeneratedClassInHierarchy(const UClass* InClass, TFunctionRef<bool(const UBlueprintGeneratedClass*)> InFunc)
{
	bool bNoErrors = true;
	while (const UBlueprintGeneratedClass* BPGClass = Cast<const UBlueprintGeneratedClass>(InClass))
	{
		if (!InFunc(BPGClass))
		{
			return bNoErrors;
		}

#if WITH_EDITORONLY_DATA
		// A cooked class has already been validated and will not have a source Blueprint asset.
		if (!BPGClass->bCooked)
		{
			const UBlueprint* BP = Cast<const UBlueprint>(BPGClass->ClassGeneratedBy);
			bNoErrors &= (NULL != BP) && (BP->Status != BS_Error);
		}
#endif
		InClass = BPGClass->GetSuperClass();
	}
	return bNoErrors;
}

UActorComponent* UBlueprintGeneratedClass::FindComponentTemplateByName(const FName& TemplateName) const
{
	for(int32 i = 0; i < ComponentTemplates.Num(); i++)
	{
		UActorComponent* Template = ComponentTemplates[i];
		if(Template != NULL && Template->GetFName() == TemplateName)
		{
			return Template;
		}
	}

	return NULL;
}

void UBlueprintGeneratedClass::CreateTimelineComponent(AActor* Actor, const UTimelineTemplate* TimelineTemplate)
{
	if (!IsValid(Actor)
		|| !TimelineTemplate
		|| Actor->IsTemplate())
	{
		return;
	}

	FName NewName = TimelineTemplate->GetVariableName();
	UTimelineComponent* NewTimeline = NewObject<UTimelineComponent>(Actor, NewName);
	NewTimeline->CreationMethod = EComponentCreationMethod::UserConstructionScript; // Indicate it comes from a blueprint so it gets cleared when we rerun construction scripts
	Actor->BlueprintCreatedComponents.Add(NewTimeline); // Add to array so it gets saved
	NewTimeline->SetNetAddressable();	// This component has a stable name that can be referenced for replication

	NewTimeline->SetPropertySetObject(Actor); // Set which object the timeline should drive properties on
	NewTimeline->SetDirectionPropertyName(TimelineTemplate->GetDirectionPropertyName());

	NewTimeline->SetTimelineLength(TimelineTemplate->TimelineLength); // copy length
	NewTimeline->SetTimelineLengthMode(TimelineTemplate->LengthMode);

	NewTimeline->PrimaryComponentTick.TickGroup = TimelineTemplate->TimelineTickGroup;

	// Find property with the same name as the template and assign the new Timeline to it
	UClass* ActorClass = Actor->GetClass();
	FObjectPropertyBase* Prop = FindFProperty<FObjectPropertyBase>(ActorClass, TimelineTemplate->GetVariableName());
	if (Prop)
	{
		Prop->SetObjectPropertyValue_InContainer(Actor, NewTimeline);
	}

	// Event tracks
	// In the template there is a track for each function, but in the runtime Timeline each key has its own delegate, so we fold them together
	for (int32 TrackIdx = 0; TrackIdx < TimelineTemplate->EventTracks.Num(); TrackIdx++)
	{
		const FTTEventTrack* EventTrackTemplate = &TimelineTemplate->EventTracks[TrackIdx];
		if (EventTrackTemplate->CurveKeys != nullptr)
		{
			// Create delegate for all keys in this track
			FScriptDelegate EventDelegate;
			EventDelegate.BindUFunction(Actor, EventTrackTemplate->GetFunctionName());

			// Create an entry in Events for each key of this track
			for (auto It(EventTrackTemplate->CurveKeys->FloatCurve.GetKeyIterator()); It; ++It)
			{
				NewTimeline->AddEvent(It->Time, FOnTimelineEvent(EventDelegate));
			}
		}
	}

	// Float tracks
	for (int32 TrackIdx = 0; TrackIdx < TimelineTemplate->FloatTracks.Num(); TrackIdx++)
	{
		const FTTFloatTrack* FloatTrackTemplate = &TimelineTemplate->FloatTracks[TrackIdx];
		if (FloatTrackTemplate->CurveFloat != NULL)
		{
			NewTimeline->AddInterpFloat(FloatTrackTemplate->CurveFloat, FOnTimelineFloat(), FloatTrackTemplate->GetPropertyName(), FloatTrackTemplate->GetTrackName());
		}
	}

	// Vector tracks
	for (int32 TrackIdx = 0; TrackIdx < TimelineTemplate->VectorTracks.Num(); TrackIdx++)
	{
		const FTTVectorTrack* VectorTrackTemplate = &TimelineTemplate->VectorTracks[TrackIdx];
		if (VectorTrackTemplate->CurveVector != NULL)
		{
			NewTimeline->AddInterpVector(VectorTrackTemplate->CurveVector, FOnTimelineVector(), VectorTrackTemplate->GetPropertyName(), VectorTrackTemplate->GetTrackName());
		}
	}

	// Linear color tracks
	for (int32 TrackIdx = 0; TrackIdx < TimelineTemplate->LinearColorTracks.Num(); TrackIdx++)
	{
		const FTTLinearColorTrack* LinearColorTrackTemplate = &TimelineTemplate->LinearColorTracks[TrackIdx];
		if (LinearColorTrackTemplate->CurveLinearColor != NULL)
		{
			NewTimeline->AddInterpLinearColor(LinearColorTrackTemplate->CurveLinearColor, FOnTimelineLinearColor(), LinearColorTrackTemplate->GetPropertyName(), LinearColorTrackTemplate->GetTrackName());
		}
	}

	// Set up delegate that gets called after all properties are updated
	FScriptDelegate UpdateDelegate;
	UpdateDelegate.BindUFunction(Actor, TimelineTemplate->GetUpdateFunctionName());
	NewTimeline->SetTimelinePostUpdateFunc(FOnTimelineEvent(UpdateDelegate));

	// Set up finished delegate that gets called after all properties are updated
	FScriptDelegate FinishedDelegate;
	FinishedDelegate.BindUFunction(Actor, TimelineTemplate->GetFinishedFunctionName());
	NewTimeline->SetTimelineFinishedFunc(FOnTimelineEvent(FinishedDelegate));

	NewTimeline->RegisterComponent();

	// Start playing now, if desired
	if (TimelineTemplate->bAutoPlay)
	{
		// Needed for autoplay timelines in cooked builds, since they won't have Activate() called via the Play call below
		NewTimeline->bAutoActivate = true;
		NewTimeline->Play();
	}

	// Set to loop, if desired
	if (TimelineTemplate->bLoop)
	{
		NewTimeline->SetLooping(true);
	}

	// Set replication, if desired
	if (TimelineTemplate->bReplicated)
	{
		NewTimeline->SetIsReplicated(true);
	}

	// Set replication, if desired
	if (TimelineTemplate->bIgnoreTimeDilation)
	{
		NewTimeline->SetIgnoreTimeDilation(true);
	}
}

void UBlueprintGeneratedClass::CreateComponentsForActor(const UClass* ThisClass, AActor* Actor)
{
	check(ThisClass && Actor);
	if (Actor->IsTemplate() || !IsValid(Actor))
	{
		return;
	}

	if (const UBlueprintGeneratedClass* BPGC = Cast<const UBlueprintGeneratedClass>(ThisClass))
	{
		for (UTimelineTemplate* TimelineTemplate : BPGC->Timelines)
		{
			// Not fatal if NULL, but shouldn't happen and ignored if not wired up in graph
			if (TimelineTemplate)
			{
				CreateTimelineComponent(Actor, TimelineTemplate);
			}
		}
	}
}

bool UBlueprintGeneratedClass::UseFastPathComponentInstancing()
{
	return bHasCookedComponentInstancingData && FPlatformProperties::RequiresCookedData() && !GBlueprintComponentInstancingFastPathDisabled;
}

uint8* UBlueprintGeneratedClass::GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const
{
	if (Obj && UsePersistentUberGraphFrame() && UberGraphFramePointerProperty && UberGraphFunction)
	{
		if (UberGraphFunction == FuncToCheck)
		{
			FPointerToUberGraphFrame* PointerToUberGraphFrame = UberGraphFramePointerProperty->ContainerPtrToValuePtr<FPointerToUberGraphFrame>(Obj);
			checkSlow(PointerToUberGraphFrame);
			ensure(PointerToUberGraphFrame->RawPointer);
			return PointerToUberGraphFrame->RawPointer;
		}
	}
	UClass* ParentClass = GetSuperClass();
	checkSlow(ParentClass);
	return ParentClass->GetPersistentUberGraphFrame(Obj, FuncToCheck);
}

void UBlueprintGeneratedClass::CreatePersistentUberGraphFrame(UObject* Obj, bool bCreateOnlyIfEmpty, bool bSkipSuperClass, UClass* OldClass) const
{
#if WITH_EDITORONLY_DATA
	/** Macros should not create uber graph frames as they have no uber graph. If UBlueprints are cooked out the macro class probably does not exist as well */
	UBlueprint* Blueprint = Cast<UBlueprint>(ClassGeneratedBy);
	if (Blueprint && Blueprint->BlueprintType == BPTYPE_MacroLibrary)
	{
		return;
	}
#endif

	ensure(!UberGraphFramePointerProperty == !UberGraphFunction);
	if (Obj && UsePersistentUberGraphFrame() && UberGraphFramePointerProperty && UberGraphFunction)
	{
		FPointerToUberGraphFrame* PointerToUberGraphFrame = UberGraphFramePointerProperty->ContainerPtrToValuePtr<FPointerToUberGraphFrame>(Obj);
		check(PointerToUberGraphFrame);

		if ( !ensureMsgf(bCreateOnlyIfEmpty || !PointerToUberGraphFrame->RawPointer
			, TEXT("Attempting to recreate an object's UberGraphFrame when the previous one was not properly destroyed (transitioning '%s' from '%s' to '%s'). We'll attempt to free the frame memory, but cannot clean up its properties (this may result in leaks and undesired side effects).")
			, *Obj->GetPathName()
			, (OldClass == nullptr) ? TEXT("<NULL>") : *OldClass->GetName()
			, *GetName()) )
		{
			FMemory::Free(PointerToUberGraphFrame->RawPointer);
			PointerToUberGraphFrame->RawPointer = nullptr;
		}
		
		if (!PointerToUberGraphFrame->RawPointer)
		{
			uint8* FrameMemory = NULL;
			const bool bUberGraphFunctionIsReady = UberGraphFunction->HasAllFlags(RF_LoadCompleted); // is fully loaded
			if (bUberGraphFunctionIsReady)
			{
				INC_MEMORY_STAT_BY(STAT_PersistentUberGraphFrameMemory, UberGraphFunction->GetStructureSize());
				FrameMemory = (uint8*)FMemory::Malloc(UberGraphFunction->GetStructureSize(), UberGraphFunction->GetMinAlignment());

				FMemory::Memzero(FrameMemory, UberGraphFunction->GetStructureSize());
				for (FProperty* Property = UberGraphFunction->PropertyLink; Property; Property = Property->PropertyLinkNext)
				{
					Property->InitializeValue_InContainer(FrameMemory);
				}
			}
			else
			{
				UE_LOGF(LogBlueprint, Verbose, "Function '%ls' is not ready to create frame for '%ls'",
					*GetPathNameSafe(UberGraphFunction), *GetPathNameSafe(Obj));
			}
			PointerToUberGraphFrame->RawPointer = FrameMemory;
#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
			PointerToUberGraphFrame->UberGraphFunctionKey = UberGraphFunctionKey;
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
		}
	}

	if (!bSkipSuperClass)
	{
		UClass* ParentClass = GetSuperClass();
		checkSlow(ParentClass);
		ParentClass->CreatePersistentUberGraphFrame(Obj, bCreateOnlyIfEmpty);
	}
}

void UBlueprintGeneratedClass::DestroyPersistentUberGraphFrame(UObject* Obj, bool bSkipSuperClass) const
{
	ensure(!UberGraphFramePointerProperty == !UberGraphFunction);
	if (Obj && UsePersistentUberGraphFrame() && UberGraphFramePointerProperty && UberGraphFunction)
	{
		FPointerToUberGraphFrame* PointerToUberGraphFrame = UberGraphFramePointerProperty->ContainerPtrToValuePtr<FPointerToUberGraphFrame>(Obj);
		checkSlow(PointerToUberGraphFrame);
		uint8* FrameMemory = PointerToUberGraphFrame->RawPointer;
		PointerToUberGraphFrame->RawPointer = NULL;
		if (FrameMemory)
		{
			for (FProperty* Property = UberGraphFunction->PropertyLink; Property; Property = Property->PropertyLinkNext)
			{
				Property->DestroyValue_InContainer(FrameMemory);
			}
			FMemory::Free(FrameMemory);
			DEC_MEMORY_STAT_BY(STAT_PersistentUberGraphFrameMemory, UberGraphFunction->GetStructureSize());
		}
		else
		{
			UE_LOGF(LogBlueprint, Log, "Object '%ls' had no Uber Graph Persistent Frame", *GetPathNameSafe(Obj));
		}
	}

	if (!bSkipSuperClass)
	{
		UClass* ParentClass = GetSuperClass();
		checkSlow(ParentClass);

		if (IsValid(ParentClass))
		{
			ParentClass->DestroyPersistentUberGraphFrame(Obj);
		}
	}
}

void UBlueprintGeneratedClass::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	
	// Super handles parent class and fields
	OutDeps.Add(GetSuperClass()->GetDefaultObject());

	if (UberGraphFunction)
	{
		OutDeps.Add(UberGraphFunction);
	}
	
	UObject *CDO = GetDefaultObject();
	if (CDO)
	{
		ForEachObjectWithOuter(CDO, [&OutDeps](UObject* SubObj)
		{
			if (SubObj->HasAnyFlags(RF_DefaultSubObject | RF_ArchetypeObject))
			{
				OutDeps.Add(SubObj->GetClass());
				OutDeps.Add(SubObj->GetArchetype());
			}
		});
	}

	if (InheritableComponentHandler)
	{
		OutDeps.Add(InheritableComponentHandler);
	}

	if (SimpleConstructionScript)
	{
		OutDeps.Add(SimpleConstructionScript);
	}
}

void UBlueprintGeneratedClass::GetDefaultObjectPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetDefaultObjectPreloadDependencies(OutDeps);
	
	// Add the super classes CDO, as it needs to be in place to properly serialize.
	OutDeps.Add(GetSuperClass()->GetDefaultObject());

	// Ensure that BPGC-owned component templates (archetypes) are loaded prior to CDO serialization in order to support the following use cases:
	//
	//	1) When the "fast path" component instancing optimization is enabled, we generate a cached delta binary at BPGC load time that we then deserialize into
	//	   new component instances after we spawn them at runtime. Generating the cached delta requires component templates to be loaded so that we can use them
	//	   as the basis for delta serialization. However, we cannot add them a preload dependency of the class without introducing a cycle, so we add them as a
	//	   preload dependency on the CDO here instead.
	//	2) When Blueprint nativization is enabled, any Blueprint class assets that are not converted to C++ may still inherit from a Blueprint class asset that is
	//	   converted to C++. In that case, the non-nativized child Blueprint class may still inherit one or more SCS nodes from the parent class. However, when
	//	   we nativize a Blueprint class, we convert the class-owned SCS component templates into CDO-owned default subobjects. In the non-nativized child Blueprint
	//	   class, these remain stored in the ICH as override templates. In order to ensure that the inherited default subobject in the CDO reflects the defaults that
	//	   are recorded into the override template, we bake out the list of changed properties at cook time and then use it to also generate a cached delta binary
	//	   when the non-nativized BPGC child asset is loaded in the cooked build. We then use binary serialization to update the default subobject instance (see
	//	   CheckAndApplyComponentTemplateOverrides). That must occur prior to serializing instances of the non-nativized BPGC so that delta serialization works
	//	   correctly, so adding them as preload dependencies here ensures that the override templates will all be loaded prior to serialization of the CDO.

	// Walk up the SCS inheritance hierarchy and add component templates (archetypes). This may include override templates contained in the ICH for inherited SCS nodes.
	UBlueprintGeneratedClass* CurrentBPClass = this;
	while (CurrentBPClass)
	{
		if (CurrentBPClass->SimpleConstructionScript)
		{
			const TArray<USCS_Node*>& AllSCSNodes = CurrentBPClass->SimpleConstructionScript->GetAllNodes();
			for (USCS_Node* SCSNode : AllSCSNodes)
			{
				// An SCS node that's owned by this class must also be considered a preload dependency since we will access its serialized template reference property. Any SCS
				// nodes that are inherited from a parent class will reference templates through the ICH instead, and that's already a preload dependency on the BP class itself.
				if (CurrentBPClass == this)
				{
					OutDeps.Add(SCSNode);
				}

				OutDeps.Add(SCSNode->GetActualComponentTemplate(this));
			}
		}

		CurrentBPClass = Cast<UBlueprintGeneratedClass>(CurrentBPClass->GetSuperClass());
	}

	// Also add UCS/AddComponent node templates (archetypes).
	for (UActorComponent* ComponentTemplate : ComponentTemplates)
	{
		if (ComponentTemplate)
		{
			OutDeps.Add(ComponentTemplate);
		}
	}

	if (GetAllowNativeComponentClassOverrides())
	{
		// Add the classes that will be used for overriding components defined in base classes
		for (const FBPComponentClassOverride& Override : ComponentClassOverrides)
		{
			if (Override.ComponentClass)
			{
				OutDeps.Add(const_cast<UClass*>(Override.ComponentClass.Get()));
			}
		}
	}

	// Add objects related to the sparse class data struct
	if (SparseClassDataStruct)
	{
		// Add the sparse class data struct, as it is serialized as part of the CDO
		OutDeps.Add(SparseClassDataStruct);

		// Also add the sparse class data archetype as it will be copied into any newly created sparse class data 
		// pre-serialization
		if(UScriptStruct* SparseClassDataArchetype = GetSparseClassDataArchetypeStruct())
		{
			OutDeps.Add(SparseClassDataArchetype);
		}

		// Add anything that the sparse class data also depends on
		const void* SparseClassDataToUse = GetSparseClassData(EGetSparseClassDataMethod::ArchetypeIfNull);
		if (UScriptStruct::ICppStructOps* CppStructOps = SparseClassDataStruct->GetCppStructOps())
		{
			CppStructOps->GetPreloadDependencies(const_cast<void*>(SparseClassDataToUse), OutDeps);
		}
		// The iterator will recursively loop through all structs in structs/containers too.
		for (TPropertyValueIterator<FStructProperty> It(SparseClassDataStruct, SparseClassDataToUse); It; ++It)
		{
			const UScriptStruct* StructType = It.Key()->Struct;
			if (UScriptStruct::ICppStructOps* CppStructOps = StructType->GetCppStructOps())
			{
				void* StructDataPtr = const_cast<void*>(It.Value());
				CppStructOps->GetPreloadDependencies(StructDataPtr, OutDeps);
			}
		}
	}
}

bool UBlueprintGeneratedClass::NeedsLoadForServer() const
{
	// This logic can't be used for targets that use editor content because UBlueprint::NeedsLoadForEditorGame
	// returns true and forces all UBlueprints to be loaded for -game or -server runs. The ideal fix would be
	// to remove UBlueprint::NeedsLoadForEditorGame, after that it would be nice if we could just implement
	// UBlueprint::NeedsLoadForEditorGame here, but we can't because then our CDO doesn't get loaded. We *could*
	// fix that behavior, but instead I'm just abusing IsRunningCommandlet() so that this logic only runs during cook:
	if (IsRunningCommandlet() && !HasAnyFlags(RF_ClassDefaultObject))
	{
		if (ensure(GetSuperClass()) && !GetSuperClass()->NeedsLoadForServer())
		{
			return false;
		}
		if (ensure(GetDefaultObject(false)) && !GetDefaultObject(false)->NeedsLoadForServer())
		{
			return false;
		}
	}
	return Super::NeedsLoadForServer();
}

bool UBlueprintGeneratedClass::NeedsLoadForClient() const
{
	// This logic can't be used for targets that use editor content because UBlueprint::NeedsLoadForEditorGame
	// returns true and forces all UBlueprints to be loaded for -game or -server runs. The ideal fix would be
	// to remove UBlueprint::NeedsLoadForEditorGame, after that it would be nice if we could just implement
	// UBlueprint::NeedsLoadForEditorGame here, but we can't because then our CDO doesn't get loaded. We *could*
	// fix that behavior, but instead I'm just abusing IsRunningCommandlet() so that this logic only runs during cook:
	if (IsRunningCommandlet() && !HasAnyFlags(RF_ClassDefaultObject))
	{
		if (ensure(GetSuperClass()) && !GetSuperClass()->NeedsLoadForClient())
		{
			return false;
		}
		if (ensure(GetDefaultObject(false)) && !GetDefaultObject(false)->NeedsLoadForClient())
		{
			return false;
		}
	}
	return Super::NeedsLoadForClient();
}

bool UBlueprintGeneratedClass::NeedsLoadForEditorGame() const
{
	return true;
}

bool UBlueprintGeneratedClass::CanBeClusterRoot() const
{
	// We don't want to cluster level BPs with the rest of the contents of the map 
	return !GetOutermost()->ContainsMap();
}

#if WITH_EDITOR
UClass* UBlueprintGeneratedClass::RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO)
{
	if (HasAnyFlags(RF_BeingRegenerated))
	{
		if (ensure(GetDefaultObject(false)))
		{
			UBlueprint::ForceLoadMembers(this);
			UBlueprint::ForceLoadMembers(GetDefaultObject(false));
		}

		if (SimpleConstructionScript)
		{
			FBlueprintEditorUtils::PreloadConstructionScript(SimpleConstructionScript);
		}

		if (InheritableComponentHandler)
		{
			InheritableComponentHandler->PreloadAll();
		}

		// Make BPGC from a cooked package standalone so it doesn't get GCed
		if (GEditor && bCooked)
		{
			SetFlags(RF_Standalone);
		}

		// If this is a cooked class, warn about any uncooked classes appearing above it in the inheritance hierarchy.
		if(bCooked)
		{
			const UBlueprintGeneratedClass* CurrentBPGC = this;
			while (const UBlueprintGeneratedClass* ParentBPGC = Cast<const UBlueprintGeneratedClass>(CurrentBPGC->GetSuperClass()))
			{
				if (!ParentBPGC->bCooked)
				{
					UE_LOGF(LogBlueprint, Warning, "%ls: found an uncooked class (%ls) as an ancestor of a cooked class (%ls) in the hierarchy.",
						*this->GetPathName(),
						*ParentBPGC->GetPathName(),
						*CurrentBPGC->GetPathName());
				}

				CurrentBPGC = ParentBPGC;
			}
		}
	}

	return this;
}

void UBlueprintGeneratedClass::PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext)
{
	Super::PreSaveRoot(ObjectSaveContext);

	auto ShouldCookBlueprintPropertyGuids = [this]() -> bool
	{
		switch (GetDefault<UCookerSettings>()->BlueprintPropertyGuidsCookingMethod)
		{
		case EBlueprintPropertyGuidsCookingMethod::EnabledBlueprintsOnly:
			if (const UBlueprint* BP = Cast<UBlueprint>(ClassGeneratedBy))
			{
				return BP->ShouldCookPropertyGuids();
			}
			break;

		case EBlueprintPropertyGuidsCookingMethod::AllBlueprints:
			return true;

		case EBlueprintPropertyGuidsCookingMethod::Disabled:
		default:
			break;
		}

		return false;
	};

	if (ObjectSaveContext.IsCooking() && ShouldCookBlueprintPropertyGuids())
	{
		CookedPropertyGuids = PropertyGuids;

		if (GetDefault<UCookerSettings>()->BlueprintPropertyGuidsCookingMethod == EBlueprintPropertyGuidsCookingMethod::EnabledBlueprintsOnly)
		{
			// Ensure that we also have the GUIDs from our parent classes available (if they're not cooking their own GUIDs)
			for (UBlueprintGeneratedClass* ParentBPGC = Cast<UBlueprintGeneratedClass>(GetSuperClass()); ParentBPGC; ParentBPGC = Cast<UBlueprintGeneratedClass>(ParentBPGC->GetSuperClass()))
			{
				const UBlueprint* ParentBP = Cast<UBlueprint>(ParentBPGC->ClassGeneratedBy);
				if (!ParentBP || ParentBP->ShouldCookPropertyGuids())
				{
					break;
				}
				CookedPropertyGuids.Append(ParentBPGC->PropertyGuids);
			}
		}
	}
	else
	{
		CookedPropertyGuids.Reset();
	}

	if (ObjectSaveContext.IsCooking() && (ObjectSaveContext.GetSaveFlags() & SAVE_Optional))
	{
		UClassCookedMetaData* CookedMetaData = NewCookedMetaData();
		CookedMetaData->CacheMetaData(this);

		if (!CookedMetaData->HasMetaData())
		{
			PurgeCookedMetaData();
		}
	}
	else
	{
		PurgeCookedMetaData();
	}
	
	if (UBlueprint* BP = Cast<UBlueprint>(ClassGeneratedBy))
	{
		BP->GeneratedClassPreSaveRoot(ObjectSaveContext);
	}
}

void UBlueprintGeneratedClass::PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
	Super::PostSaveRoot(ObjectSaveContext);

	CookedPropertyGuids.Reset();

	PurgeCookedMetaData();

	if (UBlueprint* BP = Cast<UBlueprint>(ClassGeneratedBy))
	{
		BP->GeneratedClassPostSaveRoot(ObjectSaveContext);
	}	
}
#endif

bool UBlueprintGeneratedClass::IsAsset() const
{
#if WITH_EDITOR
	// Don't include placeholder types that were created for missing type imports on load.
	// These allow exports to be serialized to avoid data loss, but should not be an asset.
	if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderType(this))
	{
		return false;
	}
#endif

	// UClass::IsAsset returns false; override that to return true for BlueprintGeneratedClasses,
	// but only if the instance satisfies the regular definition of an asset (RF_Public, not transient, etc)
	// and only if it is the active BPGC matching the Blueprint
	return UObject::IsAsset() && !HasAnyClassFlags(CLASS_NewerVersionExists);
}

void UBlueprintGeneratedClass::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	Super::Link(Ar, bRelinkExistingProperties);

	if (UsePersistentUberGraphFrame())
	{
		if (UberGraphFunction)
		{
			Ar.Preload(UberGraphFunction);

			for (FStructProperty* Property : TFieldRange<FStructProperty>(this, EFieldIteratorFlags::ExcludeSuper))
			{
				if (Property->GetFName() == GetUberGraphFrameName())
				{
					UberGraphFramePointerProperty = Property;
					break;
				}
			}
			checkSlow(UberGraphFramePointerProperty);
		}
	}

	AssembleReferenceTokenStream(true);

	// If our parent class needs dynamic instancing, this class should also inherit that trait.
	bNeedsDynamicSubobjectInstancing = bSupportsDynamicInstancedReference || GetSuperClass()->ShouldUseDynamicSubobjectInstancing();

	// If our parent class needs to be instanced on load, this class should also inherit that trait.
	bNeedsPostLoadSubobjectInstancing = GetSuperClass()->RequiresPostLoadSubobjectInstancing();
}

void UBlueprintGeneratedClass::PurgeClass(bool bRecompilingOnLoad)
{
	Super::PurgeClass(bRecompilingOnLoad);

	UberGraphFramePointerProperty = NULL;
	UberGraphFunction = NULL;
#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	UberGraphFunctionKey = 0;
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
#if WITH_EDITORONLY_DATA
	OverridenArchetypeForCDO = NULL;

#if UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
	FastCallPairs_DEPRECATED.Empty();
#endif
	CalledFunctions.Empty();
#endif //WITH_EDITORONLY_DATA
}

void UBlueprintGeneratedClass::Bind()
{
	Super::Bind();

	if (UsePersistentUberGraphFrame() && UberGraphFunction)
	{
		CppClassStaticFunctions.SetAddReferencedObjects(&UBlueprintGeneratedClass::AddReferencedObjectsInUbergraphFrame);
	}
}

#define UE_CHECK_BLUEPRINT_REFERENCES !(UE_BUILD_TEST || UE_BUILD_SHIPPING)

class FPersistentFrameCollector final : public FReferenceCollector //-V1052
{
	FReferenceCollector& InnerCollector;
#if UE_CHECK_BLUEPRINT_REFERENCES
	const uint8* PersistentFrameDataAddr;
#endif
	UObject* Blueprint;

public:
	FPersistentFrameCollector(FReferenceCollector& Collector, uint8* Instance, UObject* InBlueprint)
	: InnerCollector(Collector)
#if UE_CHECK_BLUEPRINT_REFERENCES
	, PersistentFrameDataAddr(Instance)
#endif
	, Blueprint(InBlueprint)
	{}


	virtual void HandleObjectReference(UObject*& Object, const UObject* Referencer, const FProperty* ReferencingProperty) override
	{
		check(Referencer);

#if UE_CHECK_BLUEPRINT_REFERENCES
		const bool bIsValidObjectReference = (Object == nullptr || Object->IsValidLowLevelFast());
		if (!bIsValidObjectReference)
		{
			if (const UFunction* UberGraphFunction = Cast<UFunction>(Referencer))
			{
				const int32 PersistentFrameDataSize = UberGraphFunction->GetStructureSize();

				FString PersistentFrameDataText;
				const int32 MaxBytesToDisplayPerLine = 32;
				PersistentFrameDataText.Reserve(PersistentFrameDataSize * 2 + PersistentFrameDataSize / MaxBytesToDisplayPerLine);
				for (int32 PersistentFrameDataIdx = 0; PersistentFrameDataIdx < PersistentFrameDataSize; ++PersistentFrameDataIdx)
				{
					if (PersistentFrameDataIdx % MaxBytesToDisplayPerLine == 0)
					{
						PersistentFrameDataText += TEXT("\n");
					}

					PersistentFrameDataText += FString::Printf(TEXT("%02x "), PersistentFrameDataAddr[PersistentFrameDataIdx]);
				}

				UE_LOGF(LogUObjectGlobals, Log, "PersistentFrame: Addr=0x%016llx, Size=%d%ls",
					(int64)(PTRINT)PersistentFrameDataAddr,
					PersistentFrameDataSize,
					*PersistentFrameDataText);
			}
		}

		auto GetBlueprintObjectNameLambda = [](const UObject* Referencer) -> FString
		{
			if (const UClass* BPGC = Referencer->GetTypedOuter<UClass>())
			{
#if WITH_EDITORONLY_DATA
				if (BPGC->ClassGeneratedBy)
				{
					return BPGC->ClassGeneratedBy->GetFullName();
				}
#endif
				return BPGC->GetFullName();
			}

			return TEXT("NULL");
		};

		if (!ensureMsgf(bIsValidObjectReference
			, TEXT("Invalid object referenced by the PersistentFrame: 0x%016llx (Blueprint object: %s, ReferencingProperty: %s, Instance: %s, Address: 0x%016llx) - If you have a reliable repro for this, please contact the development team with it.")
			, (int64)(PTRINT)Object
			, *GetBlueprintObjectNameLambda(Referencer)
			, *ReferencingProperty->GetFullName()
			, *Blueprint->GetFullName()
			, (int64)(PTRINT)&Object))
		{
			// clear the property value (it's garbage)... the ubergraph-frame
			// has just lost a reference to whatever it was attempting to hold onto
			Object = nullptr;
			return;
		}
#endif

		if (Object)
		{
			// If the property that serialized us is not an object property we are in some native serializer, we have to treat these as strong
			if (!Object->HasAnyFlags(RF_StrongRefOnFrame))
			{
				if (const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(ReferencingProperty))
				{
					// This was a raw UObject* serialized by FObjectProperty, so just save the address
					if (InnerCollector.MarkWeakObjectReferenceForClearing(&Object, Blueprint))
					{
						return;
					}
				}
			}

			// This is a hard reference or we don't know what's serializing it, so serialize it normally
			InnerCollector.AddReferencedObject(ObjectPtrWrap(Object), Referencer, ReferencingProperty);
		}
	}

	virtual void HandleObjectReferences(UObject** Objects, int32 Num, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
	{
		if (Num <= 0)
		{
			return;
		}

		// HandleObjectReference only calls MarkWeakObjectReferenceForClearing on object property references
		const FProperty* InnerProperty = ReferencingProperty && ReferencingProperty->IsA<FArrayProperty>() ?
			static_cast<const FArrayProperty*>(ReferencingProperty)->Inner : ReferencingProperty;

		for (int32 Idx = 0; Idx < Num; ++Idx)
		{
			HandleObjectReference(Objects[Idx], ReferencingObject, InnerProperty);
		}
	}

	virtual bool IsIgnoringArchetypeRef() const override { return InnerCollector.IsIgnoringArchetypeRef(); }
	virtual bool IsIgnoringTransient() const override { return InnerCollector.IsIgnoringArchetypeRef(); }
};

void UBlueprintGeneratedClass::AddReferencedObjectsInUbergraphFrame(UObject* InThis, FReferenceCollector& Collector)
{
	checkSlow(InThis);
	checkSlow(InThis->GetClass());
	checkSlow(InThis->GetClass() == Cast<UBlueprintGeneratedClass>(InThis->GetClass()));

	UBlueprintGeneratedClass* BPGC = static_cast<UBlueprintGeneratedClass*>(InThis->GetClass());
	UClass* SuperClass = BPGC->GetSuperClass();
	
	while (true)
	{
		UFunction* UberGraphFunction = BPGC->UberGraphFunction;
		if (BPGC->UberGraphFramePointerProperty)
		{
			FPointerToUberGraphFrame* PointerToUberGraphFrame = BPGC->UberGraphFramePointerProperty->ContainerPtrToValuePtr<FPointerToUberGraphFrame>(InThis);
			checkSlow(PointerToUberGraphFrame)

				
			FPersistentFrameCollector ProxyCollector(Collector, PointerToUberGraphFrame->RawPointer, InThis);
			if (uint8* Instance = PointerToUberGraphFrame->RawPointer)
			{
#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
				ensureMsgf(
					PointerToUberGraphFrame->UberGraphFunctionKey == BPGC->UberGraphFunctionKey,
					TEXT("Detected key mismatch in uber graph frame for instance %s of type %s, iteration will be unsafe"),
					*InThis->GetPathName(),
					*BPGC->GetPathName()
				);
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME

#if WITH_ADDITIONAL_CRASH_CONTEXTS
				BPGCBreadcrumbsParams Params =
				{
					*InThis,
					*BPGC,
					FPlatformTLS::GetCurrentThreadId()
				};

				UE_ADD_CRASH_CONTEXT_SCOPE([&](FCrashContextExtendedWriter& Writer) { WriteBPGCBreadcrumbs(Writer, Params); });
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS
				
				// All encountered references should be treated as non-native by GC
				Collector.SetIsProcessingNativeReferences(false);
				//Collector.AddPersistentFrameReferences(*UberGraphFunction, Instance, InThis);
				ProxyCollector.AddPropertyReferences(UberGraphFunction, Instance, UberGraphFunction);
				Collector.SetIsProcessingNativeReferences(true);

			}
		}

		if (SuperClass->HasAllClassFlags(CLASS_Native))
		{
			checkSlow(Cast<UBlueprintGeneratedClass>(SuperClass) == nullptr);
			SuperClass->CallAddReferencedObjects(InThis, Collector);
			break;
		}

		checkSlow(SuperClass == Cast<UBlueprintGeneratedClass>(SuperClass));
		BPGC = static_cast<UBlueprintGeneratedClass*>(SuperClass);
		SuperClass = SuperClass->GetSuperClass();
	}
}

FName UBlueprintGeneratedClass::GetUberGraphFrameName()
{
	static const FName UberGraphFrameName("UberGraphFrame");
	return UberGraphFrameName;
}

bool UBlueprintGeneratedClass::UsePersistentUberGraphFrame()
{
	static const FBoolConfigValueHelper PersistentUberGraphFrame(TEXT("Kismet"), TEXT("bPersistentUberGraphFrame"), GEngineIni);
	return PersistentUberGraphFrame;
}

#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
static TAtomic<int32> GUberGraphSerialNumber(0);

ENGINE_API int32 IncrementUberGraphSerialNumber()
{
	return ++GUberGraphSerialNumber;
}
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME

UE::Core::FVersePath UBlueprintGeneratedClass::GetVersePath() const
{
	FTopLevelAssetPath AssetPath(this);

	FNameBuilder AssetNameBuilder;
	AssetPath.GetAssetName().ToString(AssetNameBuilder);

	FStringView AssetName = AssetNameBuilder.ToView();
	if (AssetName.EndsWith(TEXT("_C"), ESearchCase::CaseSensitive))
	{
		// The asset digest declaration name is just the blueprint name.
		AssetName.LeftChopInline(2);

		AssetPath.TrySetPath(AssetPath.GetPackageName(), FName(AssetName));
	}

	return FPackageName::GetVersePath(AssetPath);
}

void UBlueprintGeneratedClass::Serialize(FArchive& Ar)
{
#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	if (Ar.IsLoading() && 0 == (Ar.GetPortFlags() & PPF_Duplicate))
	{
		UberGraphFunctionKey = IncrementUberGraphSerialNumber();
	}
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME

	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if ((Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::BPGCCookedEditorTags) &&
		Ar.IsFilterEditorOnly())
	{
#if !WITH_EDITORONLY_DATA
		FEditorTags CookedEditorTags; // Unused at runtime
#else
		CookedEditorTags.Reset();
		if (Ar.IsSaving())
		{
			GetEditorTags(CookedEditorTags);
		}
#endif
		Ar << CookedEditorTags;
	}

	if (Ar.IsLoading() && 0 == (Ar.GetPortFlags() & PPF_Duplicate))
	{
		UPackage* Package = GetOutermost();
		if (Package && Package->HasAnyPackageFlags(PKG_ForDiffing))
		{
			// If this is a diff package, set class to deprecated. This happens here to make sure it gets hit in all load cases
			ClassFlags |= CLASS_Deprecated;
		}
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsSaving() && Ar.IsCooking() && Ar.IsObjectReferenceCollector())
	{
		// The UBlueprint class and its subobjects have imports that we need to include at runtime,
		// but we exclude the Blueprint and its subobject from the saved cooked package.
		// Find all imported packages from the Blueprint and its subobjects and declare them as
		// used-in-game imports of the cooked package by serializing them as SoftObjectPaths.
		FImportExportCollector Collector(this->GetPackage());
		Collector.SetSavePackageData(Ar.GetSavePackageData());
		Collector.AddExportToIgnore(this);
		Collector.SetFilterEditorOnly(Ar.IsFilterEditorOnly());
		UObject* ClassGeneratedByPtr = ClassGeneratedBy;
		Collector.SetCallbackIsEditorOnlyObjectAllowed(
			[ClassGeneratedByPtr](const UObject* Object)
			{
				return Object == ClassGeneratedByPtr || Object->IsIn(ClassGeneratedByPtr);
			});
		Collector.SerializeObjectAndReferencedExports(ClassGeneratedByPtr);
		for (const TPair<FName, ESoftObjectPathCollectType>& Pair : Collector.GetImportedPackages())
		{
			if (Pair.Value != ESoftObjectPathCollectType::AlwaysCollect)
			{
				continue;
			}
			FName PackageName = Pair.Key;
			if (FPackageName::IsScriptPackage(WriteToString<256>(PackageName)))
			{
				// Ignore native imports; we don't need to mark them for cooking
				continue;
			}
			FSoftObjectPath PackageSoftPath(FTopLevelAssetPath(PackageName, NAME_None));
			Ar << PackageSoftPath;
		}
	}
#endif
}

void UBlueprintGeneratedClass::GetLifetimeBlueprintReplicationList(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	uint32 PropertiesLeft = NumReplicatedProperties;

	for (TFieldIterator<FProperty> It(this, EFieldIteratorFlags::ExcludeSuper); It && PropertiesLeft > 0; ++It)
	{
		FProperty * Prop = *It;
		if (Prop != NULL && Prop->GetPropertyFlags() & CPF_Net)
		{
			PropertiesLeft--;
			
			OutLifetimeProps.AddUnique(FLifetimeProperty(Prop->RepIndex, Prop->GetBlueprintReplicationCondition(), REPNOTIFY_OnChanged, PUSH_MAKE_BP_PROPERTIES_PUSH_MODEL()));
		}
	}

	UBlueprintGeneratedClass* SuperBPClass = Cast<UBlueprintGeneratedClass>(GetSuperStruct());
	if (SuperBPClass != NULL)
	{
		SuperBPClass->GetLifetimeBlueprintReplicationList(OutLifetimeProps);
	}
}

FBlueprintCookedComponentInstancingData::~FBlueprintCookedComponentInstancingData()
{
	DEC_MEMORY_STAT_BY(STAT_BPCompInstancingFastPathMemory, CachedPropertyData.GetAllocatedSize());
	DEC_MEMORY_STAT_BY(STAT_BPCompInstancingFastPathMemory, CachedPropertyListForSerialization.GetAllocatedSize());
}

void FBlueprintCookedComponentInstancingData::BuildCachedPropertyList(FCustomPropertyListNode** CurrentNode, const UStruct* CurrentScope, int32* CurrentSourceIdx) const
{
	int32 LocalSourceIdx = 0;

	if (CurrentSourceIdx == nullptr)
	{
		CurrentSourceIdx = &LocalSourceIdx;
	}

	// The serialized list is stored linearly, so stop iterating once we no longer match the scope (this indicates that we've finished parsing out "sub" properties for a UStruct).
	while (*CurrentSourceIdx < ChangedPropertyList.Num() && ChangedPropertyList[*CurrentSourceIdx].PropertyScope == CurrentScope)
	{
		// Find changed property by name/scope.
		const FBlueprintComponentChangedPropertyInfo& ChangedPropertyInfo = ChangedPropertyList[(*CurrentSourceIdx)++];
		FProperty* Property = nullptr;
		const UStruct* PropertyScope = CurrentScope;
		while (!Property && PropertyScope)
		{
			Property = FindFProperty<FProperty>(PropertyScope, ChangedPropertyInfo.PropertyName);
			PropertyScope = PropertyScope->GetSuperStruct();
		}

		// Create a new node to hold property info.
		FCustomPropertyListNode* NewNode = new FCustomPropertyListNode(Property, ChangedPropertyInfo.ArrayIndex);
		CachedPropertyListForSerialization.Add(NewNode);

		// Link the new node into the current property list.
		if (CurrentNode)
		{
			*CurrentNode = NewNode;
		}

		// If this is a UStruct property, recursively build a sub-property list.
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			BuildCachedPropertyList(&NewNode->SubPropertyList, StructProperty->Struct, CurrentSourceIdx);
		}
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			// If this is an array property, recursively build a sub-property list.
			BuildCachedArrayPropertyList(ArrayProperty, &NewNode->SubPropertyList, CurrentSourceIdx);
		}

		// Advance current location to the next linked node.
		CurrentNode = &NewNode->PropertyListNext;
	}
}

void FBlueprintCookedComponentInstancingData::BuildCachedArrayPropertyList(const FArrayProperty* ArrayProperty, FCustomPropertyListNode** ArraySubPropertyNode, int32* CurrentSourceIdx) const
{
	// Build the array property's sub-property list. An empty name field signals the end of the changed array property list.
	while (*CurrentSourceIdx < ChangedPropertyList.Num() &&
		(ChangedPropertyList[*CurrentSourceIdx].PropertyName == NAME_None
			|| ChangedPropertyList[*CurrentSourceIdx].PropertyName == ArrayProperty->GetFName()))
	{
		const FBlueprintComponentChangedPropertyInfo& ChangedArrayPropertyInfo = ChangedPropertyList[(*CurrentSourceIdx)++];
		FProperty* InnerProperty = ChangedArrayPropertyInfo.PropertyName != NAME_None ? ArrayProperty->Inner : nullptr;

		*ArraySubPropertyNode = new FCustomPropertyListNode(InnerProperty, ChangedArrayPropertyInfo.ArrayIndex);
		CachedPropertyListForSerialization.Add(*ArraySubPropertyNode);

		// If this is a UStruct property, recursively build a sub-property list.
		if (const FStructProperty* InnerStructProperty = CastField<FStructProperty>(InnerProperty))
		{
			BuildCachedPropertyList(&(*ArraySubPropertyNode)->SubPropertyList, InnerStructProperty->Struct, CurrentSourceIdx);
		}
		else if (const FArrayProperty* InnerArrayProperty = CastField<FArrayProperty>(InnerProperty))
		{
			// If this is an array property, recursively build a sub-property list.
			BuildCachedArrayPropertyList(InnerArrayProperty, &(*ArraySubPropertyNode)->SubPropertyList, CurrentSourceIdx);
		}

		ArraySubPropertyNode = &(*ArraySubPropertyNode)->PropertyListNext;
	}
}

const FCustomPropertyListNode* FBlueprintCookedComponentInstancingData::GetCachedPropertyList() const
{
	FCustomPropertyListNode* PropertyListRootNode = nullptr;

	// Construct the list if necessary.
	if (CachedPropertyListForSerialization.Num() == 0 && ChangedPropertyList.Num() > 0)
	{
		CachedPropertyListForSerialization.Reserve(ChangedPropertyList.Num());

		// Kick off construction of the cached property list.
		BuildCachedPropertyList(&PropertyListRootNode, ComponentTemplateClass);

		INC_MEMORY_STAT_BY(STAT_BPCompInstancingFastPathMemory, CachedPropertyListForSerialization.GetAllocatedSize());
	}
	else if (CachedPropertyListForSerialization.Num() > 0)
	{
		PropertyListRootNode = *CachedPropertyListForSerialization.GetData();
	}

	return PropertyListRootNode;
}

void FBlueprintCookedComponentInstancingData::BuildCachedPropertyDataFromTemplate(UActorComponent* SourceTemplate)
{
	// Blueprint component instance data writer implementation.
	class FBlueprintComponentInstanceDataWriter : public FObjectWriter
	{
	public:
		FBlueprintComponentInstanceDataWriter(TArray<uint8>& InDstBytes, const FCustomPropertyListNode* InPropertyList)
			:FObjectWriter(InDstBytes)
		{
			ArCustomPropertyList = InPropertyList;
			ArUseCustomPropertyList = true;
			this->SetWantBinaryPropertySerialization(true);

			// Set this flag to emulate things that would normally happen in the SDO case when this flag is set. This is needed to ensure consistency with serialization during instancing.
			ArPortFlags |= PPF_Duplicate;
		}
	};

	checkSlow(bHasValidCookedData);
	checkSlow(SourceTemplate != nullptr);
	checkSlow(!SourceTemplate->HasAnyFlags(RF_NeedLoad));

	// Cache source template attributes needed for instancing.
	ComponentTemplateName = SourceTemplate->GetFName();
	ComponentTemplateClass = SourceTemplate->GetClass();
	ComponentTemplateFlags = SourceTemplate->GetFlags();

	// This will also load the cached property list, if necessary.
	const FCustomPropertyListNode* PropertyList = GetCachedPropertyList();

	// Make sure we don't have any previously-built data.
	if (!ensure(CachedPropertyData.Num() == 0))
	{
		DEC_MEMORY_STAT_BY(STAT_BPCompInstancingFastPathMemory, CachedPropertyData.GetAllocatedSize());

		CachedPropertyData.Empty();
	}

	// Write template data out to the "fast path" buffer. All dependencies will be loaded at this point.
	FBlueprintComponentInstanceDataWriter InstanceDataWriter(CachedPropertyData, PropertyList);
	SourceTemplate->Serialize(InstanceDataWriter);

	INC_MEMORY_STAT_BY(STAT_BPCompInstancingFastPathMemory, CachedPropertyData.GetAllocatedSize());
}

FName UBlueprintGeneratedClass::FindBlueprintPropertyNameFromGuid(const FGuid& PropertyGuid) const
{
	auto FindPropertyNameFromGuidImpl = [&PropertyGuid](const TMap<FName, FGuid>& PropertyGuidsToUse) -> FName
	{
		if (const FName* Result = PropertyGuidsToUse.FindKey(PropertyGuid))
		{
			return *Result;
		}

		return NAME_None;
	};

	if (bCooked)
	{
		return FindPropertyNameFromGuidImpl(CookedPropertyGuids);
	}

#if WITH_EDITORONLY_DATA
	return FindPropertyNameFromGuidImpl(PropertyGuids);
#else	// WITH_EDITORONLY_DATA
	return NAME_None;
#endif	// WITH_EDITORONLY_DATA
}

FGuid UBlueprintGeneratedClass::FindBlueprintPropertyGuidFromName(const FName PropertyName) const
{
	auto FindPropertyGuidFromNameImpl = [&PropertyName](const TMap<FName, FGuid>& PropertyGuidsToUse) -> FGuid
	{
		if (const FGuid* Result = PropertyGuidsToUse.Find(PropertyName))
		{
			return *Result;
		}

		return FGuid();
	};

	if (bCooked)
	{
		return FindPropertyGuidFromNameImpl(CookedPropertyGuids);
	}

#if WITH_EDITORONLY_DATA
	return FindPropertyGuidFromNameImpl(PropertyGuids);
#else	// WITH_EDITORONLY_DATA
	return FGuid();
#endif	// WITH_EDITORONLY_DATA
}

bool UBlueprintGeneratedClass::ArePropertyGuidsAvailable() const
{
	auto AreBlueprintPropertyGuidsAvailable = [this]()
	{
		if (bCooked)
		{
			return CookedPropertyGuids.Num() > 0;
		}

#if WITH_EDITORONLY_DATA
		return PropertyGuids.Num() > 0;
#else	// WITH_EDITORONLY_DATA
		return false;
#endif	// WITH_EDITORONLY_DATA
	};

	if (AreBlueprintPropertyGuidsAvailable())
	{
		return true;
	}

	if (UBlueprintGeneratedClass* Super = Cast<UBlueprintGeneratedClass>(GetSuperStruct()))
	{
		// Our parent may have guids for inherited variables
		return Super->ArePropertyGuidsAvailable();
	}

	return false;
}

FName UBlueprintGeneratedClass::FindPropertyNameFromGuid(const FGuid& PropertyGuid) const
{
	// Check parent first as it may have renamed a property since this class was last saved
	if (UBlueprintGeneratedClass* Super = Cast<UBlueprintGeneratedClass>(GetSuperStruct()))
	{
		FName RedirectedName = Super->FindPropertyNameFromGuid(PropertyGuid);
		if (!RedirectedName.IsNone())
		{
			return RedirectedName;
		}
	}

	return FindBlueprintPropertyNameFromGuid(PropertyGuid);
}

FGuid UBlueprintGeneratedClass::FindPropertyGuidFromName(const FName InName) const
{
	const FGuid FoundPropertyGuid = FindBlueprintPropertyGuidFromName(InName);
	if (FoundPropertyGuid.IsValid())
	{
		return FoundPropertyGuid;
	}

	if (UBlueprintGeneratedClass* Super = Cast<UBlueprintGeneratedClass>(GetSuperStruct()))
	{
		// Fall back to parent if this is an inherited variable
		return Super->FindPropertyGuidFromName(InName);
	}

	return FGuid();
}

void UBlueprintGeneratedClass::ForEachFieldNotify(TFunctionRef<bool(::UE::FieldNotification::FFieldId FieldId)> Callback, bool bIncludeSuper) const
{
	ensureMsgf(FieldNotifiesStartBitNumber >= 0, TEXT("The FieldNotifyStartIndex is negative. The number of field should be positive."));
	for (int32 Index = 0; Index < FieldNotifies.Num(); ++Index)
	{
		if (!Callback(UE::FieldNotification::FFieldId(FieldNotifies[Index].GetFieldName(), Index + FieldNotifiesStartBitNumber)))
		{
			return;
		}
	}
	if (bIncludeSuper)
	{
		if (UBlueprintGeneratedClass* ParentClass = Cast<UBlueprintGeneratedClass>(GetSuperClass()))
		{
			ParentClass->ForEachFieldNotify(Callback, bIncludeSuper);
		}
	}
}

#if WITH_EDITORONLY_DATA
void UBlueprintGeneratedClass::GetEditorTags(FEditorTags& Tags) const
{
	if (UBlueprint* BP = Cast<UBlueprint>(ClassGeneratedBy))
	{
		auto AddEditorTag = [BP, &Tags](FName TagName, FName PropertyName, const uint8* PropertyValueOverride = nullptr)
		{
			const FProperty* Property = FindFieldChecked<FProperty>(UBlueprint::StaticClass(), PropertyName);
			const uint8* PropertyAddr = PropertyValueOverride ? PropertyValueOverride : Property->ContainerPtrToValuePtr<uint8>(BP);
			FString PropertyValueAsText;
			Property->ExportTextItem_Direct(PropertyValueAsText, PropertyAddr, PropertyAddr, nullptr, PPF_None);
			if (!PropertyValueAsText.IsEmpty())
			{
				Tags.Add(TagName, MoveTemp(PropertyValueAsText));
			}
		};

		AddEditorTag(FBlueprintTags::BlueprintType, GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintType));
		AddEditorTag(FBlueprintTags::BlueprintDisplayName, GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintDisplayName));

		{
			// Clear the FBPInterfaceDescription Graphs because they are irrelevant to the BPGC
			TArray<FBPInterfaceDescription> GraphlessImplementedInterfaces;
			GraphlessImplementedInterfaces.Reserve(BP->ImplementedInterfaces.Num());
			for (const FBPInterfaceDescription& ImplementedInterface : BP->ImplementedInterfaces)
			{
				FBPInterfaceDescription& GraphlessImplementedInterface = GraphlessImplementedInterfaces.Emplace_GetRef(ImplementedInterface);
				GraphlessImplementedInterface.Graphs.Empty();
			}

			AddEditorTag(FBlueprintTags::ImplementedInterfaces, GET_MEMBER_NAME_CHECKED(UBlueprint, ImplementedInterfaces), (const uint8*)&GraphlessImplementedInterfaces);
		}
	}
}

TSubclassOf<UClassCookedMetaData> UBlueprintGeneratedClass::GetCookedMetaDataClass() const
{
	return UClassCookedMetaData::StaticClass();
}

UClassCookedMetaData* UBlueprintGeneratedClass::NewCookedMetaData()
{
	if (!CachedCookedMetaDataPtr)
	{
		CachedCookedMetaDataPtr = CookedMetaDataUtil::NewCookedMetaData<UClassCookedMetaData>(this, "CookedClassMetaData", GetCookedMetaDataClass());
	}
	return CachedCookedMetaDataPtr;
}

const UClassCookedMetaData* UBlueprintGeneratedClass::FindCookedMetaData()
{
	if (!CachedCookedMetaDataPtr)
	{
		CachedCookedMetaDataPtr = CookedMetaDataUtil::FindCookedMetaData<UClassCookedMetaData>(this, TEXT("CookedClassMetaData"));
	}
	return CachedCookedMetaDataPtr;
}

void UBlueprintGeneratedClass::PurgeCookedMetaData()
{
	if (CachedCookedMetaDataPtr)
	{
		CookedMetaDataUtil::PurgeCookedMetaData<UClassCookedMetaData>(CachedCookedMetaDataPtr);
	}
}

#endif //if WITH_EDITORONLY_DATA

FBlueprintDebugData::FBlueprintDebugData() = default;
FBlueprintDebugData::FBlueprintDebugData(FBlueprintDebugData&&) = default;
FBlueprintDebugData::FBlueprintDebugData(const FBlueprintDebugData&) = default;
FBlueprintDebugData& FBlueprintDebugData::operator=(FBlueprintDebugData&&) = default;
FBlueprintDebugData& FBlueprintDebugData::operator=(const FBlueprintDebugData&) = default;
FBlueprintDebugData::~FBlueprintDebugData() = default;

#if WITH_EDITORONLY_DATA

UEdGraphNode* FBlueprintDebugData::FindNodeFromUUID(int32 UUID) const
{
	if (const TWeakObjectPtr<UEdGraphNode>* pParentNode = DebugNodesAllocatedUniqueIDsMap.Find(UUID))
	{
		return pParentNode->Get();
	}
	
	return nullptr;
}

bool FBlueprintDebugData::IsValid() const
{
	return DebugNodeLineNumbers.Num() > 0;
}

UEdGraphNode* FBlueprintDebugData::FindSourceNodeFromCodeLocation(const UFunction* Function, int32 CodeOffset, bool bAllowImpreciseHit) const
{
	if (const FDebuggingInfoForSingleFunction* pFuncInfo = PerFunctionLineNumbers.Find(Function))
	{
		UEdGraphNode* Result = pFuncInfo->LineNumberToSourceNodeMap.FindRef(CodeOffset).Get();

		if ((Result == nullptr) && bAllowImpreciseHit)
		{
			for (int32 TrialOffset = CodeOffset + 1; (Result == nullptr) && (TrialOffset < Function->Script.Num()); ++TrialOffset)
			{
				Result = pFuncInfo->LineNumberToSourceNodeMap.FindRef(TrialOffset).Get();
			}
		}

		return Result;
	}

	return nullptr;
}

UEdGraphPin* FBlueprintDebugData::FindSourcePinFromCodeLocation(UFunction* Function, int32 CodeOffset) const
{
	if (const FDebuggingInfoForSingleFunction* pFuncInfo = PerFunctionLineNumbers.Find(Function))
	{
		return pFuncInfo->LineNumberToSourcePinMap.FindRef(CodeOffset).Get();
	}

	return nullptr;
}

void FBlueprintDebugData::FindAllCodeLocationsFromSourcePin(UEdGraphPin const* SourcePin, UFunction* InFunction, TArray<int32>& OutPinToCodeAssociations) const
{
	OutPinToCodeAssociations.Empty();

	if (const FDebuggingInfoForSingleFunction* pFuncInfo = PerFunctionLineNumbers.Find(InFunction))
	{
		pFuncInfo->SourcePinToLineNumbersMap.MultiFind(SourcePin, OutPinToCodeAssociations, true);
	}
}

int32 FBlueprintDebugData::FindCodeLocationFromSourcePin(UEdGraphPin const* SourcePin, UFunction* InFunction, FInt32Range InRange) const
{
	TArray<int32> PinToCodeAssociations;
	FindAllCodeLocationsFromSourcePin(SourcePin, InFunction, PinToCodeAssociations);

	for (int32 i = 0; i < PinToCodeAssociations.Num(); ++i)
	{
		if (InRange.Contains(PinToCodeAssociations[i]))
		{
			return PinToCodeAssociations[i];
		}
	}

	return INDEX_NONE;
}

void FBlueprintDebugData::FindAllCodeLocationsFromSourceNode(UEdGraphNode* SourceNode, UFunction* InFunction, TArray<int32>& OutNodeToCodeAssociations) const
{
	OutNodeToCodeAssociations.Empty();

	if (const FDebuggingInfoForSingleFunction* pFuncInfo = PerFunctionLineNumbers.Find(InFunction))
	{
		for (auto CodeLocation : pFuncInfo->LineNumberToSourceNodeMap)
		{
			if (CodeLocation.Value == SourceNode)
			{
				OutNodeToCodeAssociations.Add(CodeLocation.Key);
			}
		}
	}
}

FInt32Range FBlueprintDebugData::FindPureNodeScriptCodeRangeFromSourceNode(const UEdGraphNode* SourceNode, UFunction* InFunction) const
{
	FInt32Range Result = FInt32Range(INDEX_NONE);

	if (const FDebuggingInfoForSingleFunction* DebugInfoPtr = PerFunctionLineNumbers.Find(InFunction))
	{
		if (const FInt32Range* ValuePtr = DebugInfoPtr->PureNodeScriptCodeRangeMap.Find(MakeWeakObjectPtr(const_cast<UEdGraphNode*>(SourceNode))))
		{
			Result = *ValuePtr;
		}
	}

	return Result;
}

const TArray<TWeakObjectPtr<UEdGraphNode> >* FBlueprintDebugData::FindExpansionSourceNodesFromCodeLocation(UFunction* Function, int32 CodeOffset) const
{
	if (const FDebuggingInfoForSingleFunction* pFuncInfo = PerFunctionLineNumbers.Find(Function))
	{
		return pFuncInfo->LineNumberToTunnelInstanceSourceNodesMap.Find(CodeOffset);
	}

	return nullptr;
}

void FBlueprintDebugData::FindBreakpointInjectionSites(UEdGraphNode* Node, TArray<uint8*>& InstallSites) const
{
	TArray<int32> RecordIndices;
	DebugNodeIndexLookup.MultiFind(Node, RecordIndices, true);
	for(int i = 0; i < RecordIndices.Num(); ++i)
	{
		int32 RecordIndex = RecordIndices[i];
		if (DebugNodeLineNumbers.IsValidIndex(RecordIndex))
		{
			const FNodeToCodeAssociation& Record = DebugNodeLineNumbers[RecordIndex];
			if (UFunction* Scope = Record.Scope.Get())
			{
				if (Scope->Script.IsValidIndex(Record.Offset))
				{
					InstallSites.Add(&(Scope->Script[Record.Offset]));
				}
			}
		}
	}
}

FProperty* FBlueprintDebugData::FindClassPropertyForPin(const UEdGraphPin* Pin) const
{
	if (!Pin)
	{
		return nullptr;
	}

	TFieldPath<FProperty> PropertyPtr = DebugPinToPropertyMap.FindRef(Pin);
	if ((PropertyPtr == nullptr) && (Pin->LinkedTo.Num() > 0))
	{
		// Try checking the other side of the connection
		PropertyPtr = DebugPinToPropertyMap.FindRef(Pin->LinkedTo[0]);
	}

	return *PropertyPtr;
}

FProperty* FBlueprintDebugData::FindClassPropertyForNode(const UEdGraphNode* Node) const
{
	return *DebugObjectToPropertyMap.FindRef(MakeWeakObjectPtr(const_cast<UEdGraphNode*>(Node)));
}

void FBlueprintDebugData::RegisterNodeToCodeAssociation(UEdGraphNode* SourceNode, const TArray<TWeakObjectPtr<UEdGraphNode> >& ExpansionSourceNodes, UFunction* InFunction, int32 CodeOffset, bool bBreakpointSite)
{
	//@TODO: Nasty expansion behavior during compile time
	if (bBreakpointSite)
	{
		DebugNodeLineNumbers.Emplace(SourceNode, InFunction, CodeOffset);
		DebugNodeIndexLookup.Add(SourceNode, DebugNodeLineNumbers.Num() - 1);
	}

	FDebuggingInfoForSingleFunction& PerFuncInfo = PerFunctionLineNumbers.FindOrAdd(InFunction);
	PerFuncInfo.LineNumberToSourceNodeMap.Add(CodeOffset, SourceNode);

	if (ExpansionSourceNodes.Num() > 0)
	{
		PerFuncInfo.LineNumberToTunnelInstanceSourceNodesMap.Add(CodeOffset, ExpansionSourceNodes);
	}
}

void FBlueprintDebugData::RegisterPureNodeScriptCodeRange(UEdGraphNode* SourceNode, UFunction* InFunction, FInt32Range InPureNodeScriptCodeRange)
{
	FDebuggingInfoForSingleFunction& PerFuncInfo = PerFunctionLineNumbers.FindOrAdd(InFunction);
	PerFuncInfo.PureNodeScriptCodeRangeMap.Add(SourceNode, InPureNodeScriptCodeRange);
}

void FBlueprintDebugData::RegisterPinToCodeAssociation(UEdGraphPin const* SourcePin, UFunction* InFunction, int32 CodeOffset)
{
	FDebuggingInfoForSingleFunction& PerFuncInfo = PerFunctionLineNumbers.FindOrAdd(InFunction);
	PerFuncInfo.LineNumberToSourcePinMap.Add(CodeOffset, SourcePin);
	PerFuncInfo.SourcePinToLineNumbersMap.Add(SourcePin, CodeOffset);
}

const TMap<int32, FName>& FBlueprintDebugData::GetEntryPoints() const
{
	return EntryPoints;
}

bool FBlueprintDebugData::IsValidEntryPoint(const int32 LinkId) const
{
	return EntryPoints.Contains(LinkId);
}

void FBlueprintDebugData::RegisterEntryPoint(const int32 ScriptOffset, const FName FunctionName)
{
	EntryPoints.Add(ScriptOffset, FunctionName);
}

void FBlueprintDebugData::RegisterClassPropertyAssociation(class UObject* TrueSourceObject, class FProperty* AssociatedProperty)
{
	DebugObjectToPropertyMap.Add(TrueSourceObject, AssociatedProperty);
}

void FBlueprintDebugData::RegisterClassPropertyAssociation(const UEdGraphPin* TrueSourcePin, class FProperty* AssociatedProperty)
{
	if (TrueSourcePin)
	{
		DebugPinToPropertyMap.Add(TrueSourcePin, AssociatedProperty);
	}
}

void FBlueprintDebugData::RegisterUUIDAssociation(UEdGraphNode* TrueSourceNode, int32 UUID)
{
	DebugNodesAllocatedUniqueIDsMap.Add(UUID, TrueSourceNode);
}

UObject* FBlueprintDebugData::FindObjectThatCreatedProperty(class FProperty* AssociatedProperty) const
{
	if (const TWeakObjectPtr<UObject>* pValue = DebugObjectToPropertyMap.FindKey(AssociatedProperty))
	{
		return pValue->Get();
	}
	else
	{
		return nullptr;
	}
}

UEdGraphPin* FBlueprintDebugData::FindPinThatCreatedProperty(class FProperty* AssociatedProperty) const
{
	if (const FEdGraphPinReference* pValue = DebugPinToPropertyMap.FindKey(AssociatedProperty))
	{
		return pValue->Get();
	}
	else
	{
		return nullptr;
	}
}

void FBlueprintDebugData::GenerateReversePropertyMap(TMap<FProperty*, UObject*>& PropertySourceMap)
{
	for (TMap<TWeakObjectPtr<UObject>, TFieldPath<FProperty>>::TIterator MapIt(DebugObjectToPropertyMap); MapIt; ++MapIt)
	{
		if (UObject* SourceObj = MapIt.Key().Get())
		{
			PropertySourceMap.Add(*MapIt.Value(), SourceObj);
		}
	}
}

#endif //if WITH_EDITORONLY_DATA
