// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintInstancedStructLibrary.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "UObject/EnumProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintInstancedStructLibrary)

#define LOCTEXT_NAMESPACE "UBlueprintInstancedStructLibrary"


FInstancedStruct UBlueprintInstancedStructLibrary::MakeInstancedStruct(const int32& Value)
{
	// We should never hit this! stubs to avoid NoExport on the class.
	checkNoEntry();
	return {};
}

void UBlueprintInstancedStructLibrary::SetInstancedStructValue(FInstancedStruct& InstancedStruct, const int32& Value)
{
	// We should never hit this! stubs to avoid NoExport on the class.
	checkNoEntry();
}

void UBlueprintInstancedStructLibrary::GetInstancedStructValue(EStructUtilsResult& ExecResult, const FInstancedStruct& InstancedStruct, int32& Value)
{
	// We should never hit this! stubs to avoid NoExport on the class.
	checkNoEntry();
}

DEFINE_FUNCTION(UBlueprintInstancedStructLibrary::execMakeInstancedStruct)
{
	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	
	const FStructProperty* ValueProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	const void* ValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (!ValueProp || !ValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("InstancedStruct_MakeInvalidValueWarning", "Failed to resolve the Value for Make Instanced Struct")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);

		P_NATIVE_BEGIN;
		(*(FInstancedStruct*)RESULT_PARAM).Reset();
		P_NATIVE_END;
	}
	else
	{
		P_NATIVE_BEGIN;
		(*(FInstancedStruct*)RESULT_PARAM).InitializeAs(ValueProp->Struct, (const uint8*)ValuePtr);
		P_NATIVE_END;
	}
}

DEFINE_FUNCTION(UBlueprintInstancedStructLibrary::execSetInstancedStructValue)
{
	P_GET_STRUCT_REF(FInstancedStruct, InstancedStruct);

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	
	const FStructProperty* ValueProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	const void* ValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (!ValueProp || !ValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("InstancedStruct_SetInvalidValueWarning", "Failed to resolve the Value for Set Instanced Struct Value")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);

		P_NATIVE_BEGIN;
		InstancedStruct.Reset();
		P_NATIVE_END;
	}
	else
	{
		P_NATIVE_BEGIN;
		InstancedStruct.InitializeAs(ValueProp->Struct, (const uint8*)ValuePtr);
		P_NATIVE_END;
	}
}

DEFINE_FUNCTION(UBlueprintInstancedStructLibrary::execGetInstancedStructValue)
{
	P_GET_ENUM_REF(EStructUtilsResult, ExecResult);
	P_GET_STRUCT_REF(FInstancedStruct, InstancedStruct);

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	
	const FStructProperty* ValueProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* ValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	ExecResult = EStructUtilsResult::NotValid;

	if (!ValueProp || !ValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("InstancedStruct_GetInvalidValueWarning", "Failed to resolve the Value for Get Instanced Struct Value")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		if (InstancedStruct.IsValid() && InstancedStruct.GetScriptStruct()->IsChildOf(ValueProp->Struct))
		{
			ValueProp->Struct->CopyScriptStruct(ValuePtr, InstancedStruct.GetMemory());
			ExecResult = EStructUtilsResult::Valid;
		}
		else
		{
			ExecResult = EStructUtilsResult::NotValid;
		}
		P_NATIVE_END;
	}
}

#undef LOCTEXT_NAMESPACE
