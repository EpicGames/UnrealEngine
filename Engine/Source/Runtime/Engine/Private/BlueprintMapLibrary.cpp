// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintMapLibrary.h"
#include "Kismet/BlueprintPropertyHelpers.h"
#include "Kismet/KismetArrayLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintMapLibrary)

void UBlueprintMapLibrary::GenericMap_Add(const void* TargetMap, const FMapProperty* MapProperty, const void* KeyPtr, const void* ValuePtr)
{
	if (TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		if (MapHelper.Num() < MaxSupportedMapSize)
		{
			MapHelper.AddPair(KeyPtr, ValuePtr);
		}
		else if (!MapHelper.FindValueFromHash(KeyPtr))
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted add to map '%s' beyond the maximum supported capacity!"), *MapProperty->GetName()), ELogVerbosity::Warning, UKismetArrayLibrary::ReachedMaximumContainerSizeWarning);
		}
	}
}

bool UBlueprintMapLibrary::GenericMap_Remove(const void* TargetMap, const FMapProperty* MapProperty, const void* KeyPtr)
{
	if(TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		return MapHelper.RemovePair(KeyPtr);
	}
	return false;
}

bool UBlueprintMapLibrary::GenericMap_Find(const void* TargetMap, const FMapProperty* MapProperty, const void* KeyPtr, void* OutValuePtr)
{
	if(TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		uint8* FoundValuePtr = MapHelper.FindValueFromHash(KeyPtr);
		if (OutValuePtr)
		{
			if (FoundValuePtr)
			{
				MapProperty->ValueProp->CopyCompleteValueFromScriptVM(OutValuePtr, FoundValuePtr);
			}
			else
			{
				UE::BlueprintPropertyHelpers::ResetToDefault(OutValuePtr, MapProperty->ValueProp);
			}
		}
		return FoundValuePtr != nullptr;
	}
	return false;
}

void UBlueprintMapLibrary::GenericMap_Keys(const void* TargetMap, const FMapProperty* MapProperty, const void* TargetArray, const FArrayProperty* ArrayProperty)
{
	if(TargetMap && TargetArray && ensure(MapProperty->KeyProp->GetID() == ArrayProperty->Inner->GetID()) )
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		FScriptArrayHelper ArrayHelper(ArrayProperty, TargetArray);
		ArrayHelper.EmptyValues();

		const FProperty* InnerProp = ArrayProperty->Inner;
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			const int32 LastIndex = ArrayHelper.AddValue();
			InnerProp->CopySingleValueToScriptVM(ArrayHelper.GetRawPtr(LastIndex), MapHelper.GetKeyPtr(It));
		}
	}
}

void UBlueprintMapLibrary::GenericMap_Values(const void* TargetMap, const FMapProperty* MapProperty, const void* TargetArray, const FArrayProperty* ArrayProperty)
{	
	if(TargetMap && TargetArray && ensure(MapProperty->ValueProp->GetID() == ArrayProperty->Inner->GetID()))
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		FScriptArrayHelper ArrayHelper(ArrayProperty, TargetArray);
		ArrayHelper.EmptyValues();

		const FProperty* InnerProp = ArrayProperty->Inner;
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			const int32 LastIndex = ArrayHelper.AddValue();
			InnerProp->CopySingleValueToScriptVM(ArrayHelper.GetRawPtr(LastIndex), MapHelper.GetValuePtr(It));
		}
	}
}

int32 UBlueprintMapLibrary::GenericMap_Length(const void* TargetMap, const FMapProperty* MapProperty)
{
	if(TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		return MapHelper.Num();
	}
	return 0;
}

bool UBlueprintMapLibrary::GenericMap_IsEmpty(const void* TargetMap, const FMapProperty* MapProperty)
{
	if (TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		return MapHelper.Num() == 0;
	}
	return true;
}

bool UBlueprintMapLibrary::GenericMap_IsNotEmpty(const void* TargetMap, const FMapProperty* MapProperty)
{
	if (TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		return MapHelper.Num() > 0;
	}
	return false;
}

void UBlueprintMapLibrary::GenericMap_Clear(const void* TargetMap, const FMapProperty* MapProperty)
{
	if(TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		MapHelper.EmptyValues();
	}
}

void UBlueprintMapLibrary::GenericMap_SetMapPropertyByName(UObject* OwnerObject, FName MapPropertyName, const void* SrcMapAddr)
{
	if (OwnerObject)
	{
		if (FMapProperty* MapProp = FindFProperty<FMapProperty>(OwnerObject->GetClass(), MapPropertyName))
		{
			void* Dest = MapProp->ContainerPtrToValuePtr<void>(OwnerObject);
			MapProp->CopyValuesInternal(Dest, SrcMapAddr, 1);
		}
	}
}

void UBlueprintMapLibrary::GenericMap_GetKeyValueByIndex(const void* TargetMap, const FMapProperty* MapProperty, int32 Index, void* KeyPtr, void* ValuePtr)
{
	if (TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		const FProperty* KeyProp = MapProperty->KeyProp;
		const FProperty* ValueProp = MapProperty->ValueProp;
		const TScriptContainerIterator<FScriptMapHelper> Iterator = MapHelper.CreateIterator(Index);
		
		if (Iterator)
		{
			KeyProp->CopySingleValueToScriptVM(KeyPtr, MapHelper.GetKeyPtr(Iterator));
			ValueProp->CopySingleValueToScriptVM(ValuePtr, MapHelper.GetValuePtr(Iterator));
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to access index %d from map '%s' of length %d in '%s'!"),
				Index,
				*MapProperty->GetName(),
				MapHelper.Num(),
				*MapProperty->GetOwnerVariant().GetPathName()),
				ELogVerbosity::Warning,
				FName("GetOutOfBoundsWarning"));
		}
	}
}

int32 UBlueprintMapLibrary::GenericMap_GetLastIndex(const void* TargetMap, const FMapProperty* MapProperty)
{
	if (TargetMap)
	{
		FScriptMapHelper MapHelper(MapProperty, TargetMap);
		return MapHelper.Num() - 1;
	}

	return 0;
}
