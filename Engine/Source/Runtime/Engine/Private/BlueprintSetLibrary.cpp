// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintSetLibrary.h"
#include "Kismet/KismetArrayLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintSetLibrary)

void UBlueprintSetLibrary::GenericSet_Add(const void* TargetSet, const FSetProperty* SetProperty, const void* ItemPtr)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);
		if (SetHelper.Num() < MaxSupportedSetSize)
		{
			SetHelper.AddElement(ItemPtr);
		}
		else if (SetHelper.FindElementIndexFromHash(ItemPtr) == INDEX_NONE)
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted add to set '%s' beyond the maximum supported capacity!"), *SetProperty->GetName()), ELogVerbosity::Warning, UKismetArrayLibrary::ReachedMaximumContainerSizeWarning);
		}
	}
}

void UBlueprintSetLibrary::GenericSet_AddItems(const void* TargetSet, const FSetProperty* SetProperty, const void* TargetArray, const FArrayProperty* ArrayProperty)
{
	if(TargetSet && TargetArray)
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, TargetArray);

		const int32 Size = ArrayHelper.Num();
		for (int32 I = 0; I < Size; ++I)
		{
			GenericSet_Add(TargetSet, SetProperty, ArrayHelper.GetRawPtr(I));
		}
	}
}

bool UBlueprintSetLibrary::GenericSet_Remove(const void* TargetSet, const FSetProperty* SetProperty, const void* ItemPtr)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);
		return SetHelper.RemoveElement(ItemPtr);
	}
	return false;
}

void UBlueprintSetLibrary::GenericSet_RemoveItems(const void* TargetSet, const FSetProperty* SetProperty, const void* TargetArray, const FArrayProperty* ArrayProperty)
{
	if (TargetSet && TargetArray)
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, TargetArray);

		const int32 Size = ArrayHelper.Num();
		for (int32 I = 0; I < Size; ++I)
		{
			GenericSet_Remove(TargetSet, SetProperty, ArrayHelper.GetRawPtr(I));
		}
	}
}

void UBlueprintSetLibrary::GenericSet_ToArray(const void* TargetSet, const FSetProperty* SetProperty, void* TargetArray, const FArrayProperty* ArrayProperty)
{
	if (TargetSet && TargetArray)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);
		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			UKismetArrayLibrary::GenericArray_Add(TargetArray, ArrayProperty, SetHelper.GetElementPtr(It));
		}
	}
}

void UBlueprintSetLibrary::GenericSet_Clear(const void* TargetSet, const FSetProperty* SetProperty)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);
		SetHelper.EmptyElements();
	}
}

int32 UBlueprintSetLibrary::GenericSet_Length(const void* TargetSet, const FSetProperty* SetProperty)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);
		return SetHelper.Num();
	}

	return 0;
}

bool UBlueprintSetLibrary::GenericSet_Contains(const void* TargetSet, const FSetProperty* SetProperty, const void* ItemToFind)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);
		FProperty* ElementProp = SetProperty->ElementProp;

		return SetHelper.FindElementIndexFromHash(ItemToFind) != INDEX_NONE;
	}

	return false;
}

bool UBlueprintSetLibrary::GenericSet_IsEmpty(const void* TargetSet, const FSetProperty* SetProperty)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);

		return SetHelper.Num() == 0;
	}

	return true;
}

bool UBlueprintSetLibrary::GenericSet_IsNotEmpty(const void* TargetSet, const FSetProperty* SetProperty)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);

		return SetHelper.Num() > 0;
	}
	return false;
}

void UBlueprintSetLibrary::GenericSet_Intersect(const void* SetA, const FSetProperty* SetPropertyA, const void* SetB, const FSetProperty* SetPropertyB, const void* SetResult, const FSetProperty* SetPropertyResult)
{
	if (SetA && SetB && SetResult)
	{
		FScriptSetHelper SetHelperA(SetPropertyA, SetA);
		FScriptSetHelper SetHelperB(SetPropertyB, SetB);
		FScriptSetHelper SetHelperResult(SetPropertyResult, SetResult);
		
		SetHelperResult.EmptyElements();
		for (FScriptSetHelper::FIterator It(SetHelperA); It; ++It)
		{
			const void* EntryInA = SetHelperA.GetElementPtr(It);
			if (SetHelperB.FindElementIndexFromHash(EntryInA) != INDEX_NONE)
			{
				SetHelperResult.AddElement(EntryInA);
			}
		}
	}
}

void UBlueprintSetLibrary::GenericSet_Union(const void* SetA, const FSetProperty* SetPropertyA, const void* SetB, const FSetProperty* SetPropertyB, const void* SetResult, const FSetProperty* SetPropertyResult)
{
	if (SetA && SetB && SetResult)
	{
		FScriptSetHelper SetHelperA(SetPropertyA, SetA);
		FScriptSetHelper SetHelperB(SetPropertyB, SetB);
		FScriptSetHelper SetHelperResult(SetPropertyResult, SetResult);
		
		SetHelperResult.EmptyElements();

		for (FScriptSetHelper::FIterator It(SetHelperA); It; ++It)
		{
			SetHelperResult.AddElement(SetHelperA.GetElementPtr(It));
		}

		for (FScriptSetHelper::FIterator It(SetHelperB); It; ++It)
		{
			SetHelperResult.AddElement(SetHelperB.GetElementPtr(It));
		}
	}
}

void UBlueprintSetLibrary::GenericSet_Difference(const void* SetA, const FSetProperty* SetPropertyA, const void* SetB, const FSetProperty* SetPropertyB, const void* SetResult, const FSetProperty* SetPropertyResult)
{
	if (SetA && SetB && SetResult)
	{
		FScriptSetHelper SetHelperA(SetPropertyA, SetA);
		FScriptSetHelper SetHelperB(SetPropertyB, SetB);
		FScriptSetHelper SetHelperResult(SetPropertyResult, SetResult);

		SetHelperResult.EmptyElements();

		for (FScriptSetHelper::FIterator It(SetHelperA); It; ++It)
		{
			const void* EntryInA = SetHelperA.GetElementPtr(It);
			if (SetHelperB.FindElementIndexFromHash(EntryInA) == INDEX_NONE)
			{
				SetHelperResult.AddElement(EntryInA);
			}
		}
	}
}

void UBlueprintSetLibrary::GenericSet_SetSetPropertyByName(UObject* OwnerObject, FName SetPropertyName, const void* SrcSetAddr)
{
	if (OwnerObject)
	{
		if (FSetProperty* SetProp = FindFProperty<FSetProperty>(OwnerObject->GetClass(), SetPropertyName))
		{
			void* Dest = SetProp->ContainerPtrToValuePtr<void>(OwnerObject);
			SetProp->CopyValuesInternal(Dest, SrcSetAddr, 1);
		}
	}
}

void UBlueprintSetLibrary::GenericSet_GetItemByIndex(void* TargetSet, const FSetProperty* SetProperty, int32 Index, void* Item)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);
		const FProperty* ElementProp = SetProperty->ElementProp;
		const TScriptContainerIterator<FScriptSetHelper> Iterator = SetHelper.CreateIterator(Index);

		if (Iterator)
		{
			ElementProp->CopySingleValueToScriptVM(Item, SetHelper.GetElementPtr(Iterator));
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to access invalid index %d from set '%s' of length %d in '%s'!"),
				Index,
				*SetProperty->GetName(),
				SetHelper.Num(),
				*SetProperty->GetOwnerVariant().GetPathName()),
				ELogVerbosity::Warning,
				FName("GetOutOfBoundsWarning"));
		}
	}
}

int32 UBlueprintSetLibrary::GenericSet_GetLastIndex(const void* TargetSet, const FSetProperty* SetProperty)
{
	if (TargetSet)
	{
		FScriptSetHelper SetHelper(SetProperty, TargetSet);
		return SetHelper.Num() - 1;
	}

	return 0;
}
