// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintSpringMathLibrary.h"

#include "Math/SpringMath.h"

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintSpingMathLibrary, Log, All);

void UBlueprintSpringMathLibrary::CriticalSpringDampVector(FVector& InOutX, FVector& InOutV, const FVector& TargetX, float DeltaTime, float SmoothingTime)
{
	SpringMath::CriticalSpringDamper(InOutX, InOutV, TargetX, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::CriticalSpringDampVector2D(FVector2D& InOutX, FVector2D& InOutV, const FVector2D& TargetX, float DeltaTime, float SmoothingTime)
{
	SpringMath::CriticalSpringDamper(InOutX, InOutV, TargetX, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::CriticalSpringDampFloat(float& InOutX, float& InOutV, const float& TargetX, float DeltaTime, float SmoothingTime)
{
	SpringMath::CriticalSpringDamper(InOutX, InOutV, TargetX, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::CriticalSpringDampAngle(float& InOutAngle, float& InOutAngularVelocity, const float& TargetAngle, float DeltaTime, float SmoothingTime)
{
	float InOutAngleRadians = FMath::DegreesToRadians(InOutAngle);
	float InOutAngularVelocityRadians = FMath::DegreesToRadians(InOutAngularVelocity);
	SpringMath::CriticalSpringDamperAngle(InOutAngleRadians, InOutAngularVelocityRadians, FMath::DegreesToRadians(TargetAngle), SmoothingTime, DeltaTime);
	InOutAngle = FMath::RadiansToDegrees(InOutAngleRadians);
	InOutAngularVelocity = FMath::RadiansToDegrees(InOutAngularVelocityRadians);
}

void UBlueprintSpringMathLibrary::CriticalSpringDampQuat(FQuat& InOutRotation, FVector& InOutAngularVelocity, const FQuat& TargetRotation, float DeltaTime, float SmoothingTime)
{
	FVector InOutAngularVelocityRadians = FMath::DegreesToRadians(InOutAngularVelocity);
	SpringMath::CriticalSpringDamperQuat(InOutRotation, InOutAngularVelocityRadians, TargetRotation, SmoothingTime, DeltaTime);
	InOutAngularVelocity = FMath::RadiansToDegrees(InOutAngularVelocityRadians);
}

void UBlueprintSpringMathLibrary::CriticalSpringDampRotator(FRotator& InOutRotation, FVector& InOutAngularVelocity, const FRotator& TargetRotation, float DeltaTime, float SmoothingTime)
{
	FQuat InOutRotationQuat = InOutRotation.Quaternion();
	FVector InOutAngularVelocityRadians = FMath::DegreesToRadians(InOutAngularVelocity);
	SpringMath::CriticalSpringDamperQuat(InOutRotationQuat, InOutAngularVelocityRadians, TargetRotation.Quaternion(), SmoothingTime, DeltaTime);
	InOutRotation = InOutRotationQuat.Rotator();
	InOutAngularVelocity = FMath::RadiansToDegrees(InOutAngularVelocityRadians);
}

void UBlueprintSpringMathLibrary::CriticalSpringDampScale(FVector& InOutScale, FVector& InOutScalarVelocity, const FVector& TargetScale, float DeltaTime, float SmoothingTime)
{
	SpringMath::CriticalSpringDamperScale(InOutScale, InOutScalarVelocity, TargetScale, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::CriticalSpringDampTransform(FTransform& InOutTransform, FSpringTransformVelocity& InOutVelocity, const FTransform& TargetTransform, float DeltaTime, float SmoothingTime)
{
	FVector InOutLocation = InOutTransform.GetLocation();
	FQuat InOutRotation = InOutTransform.GetRotation();
	FVector InOutScale = InOutTransform.GetScale3D();

	CriticalSpringDampVector(InOutLocation, InOutVelocity.LinearVelocity, TargetTransform.GetLocation(), DeltaTime, SmoothingTime);
	CriticalSpringDampQuat(InOutRotation, InOutVelocity.AngularVelocity, TargetTransform.GetRotation(), DeltaTime, SmoothingTime);
	CriticalSpringDampScale(InOutScale, InOutVelocity.ScalarVelocity, TargetTransform.GetScale3D(), DeltaTime, SmoothingTime);

	InOutTransform = FTransform(InOutRotation, InOutLocation, InOutScale);
}

void UBlueprintSpringMathLibrary::CriticalDoubleSpringDampVector(FVector& InOutX, FVector& InOutV, FVector& InOutXi, FVector& InOutVi, const FVector& TargetX, float DeltaTime, float SmoothingTime)
{
	SpringMath::CriticalDoubleSpringDamper(InOutX, InOutV, InOutXi, InOutVi, TargetX, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::CriticalDoubleSpringDampVector2D(FVector2D& InOutX, FVector2D& InOutV, FVector2D& InOutXi, FVector2D& InOutVi, const FVector2D& TargetX, float DeltaTime, float SmoothingTime)
{
	SpringMath::CriticalDoubleSpringDamper(InOutX, InOutV, InOutXi, InOutVi, TargetX, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::CriticalDoubleSpringDampFloat(float& InOutX, float& InOutV, float& InOutXi, float& InOutVi, const float& TargetX, float DeltaTime, float SmoothingTime)
{
	SpringMath::CriticalDoubleSpringDamper(InOutX, InOutV, InOutXi, InOutVi, TargetX, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::CriticalDoubleSpringDampAngle(float& InOutAngle, float& InOutAngularVelocity, float& InOutIntermediateAngle, float& InOutIntermediateAngularVelocity, const float& TargetAngle, float DeltaTime, float SmoothingTime)
{
	float InOutAngleRadians = FMath::DegreesToRadians(InOutAngle);
	float InOutAngularVelocityRadians = FMath::DegreesToRadians(InOutAngularVelocity);
	float InOutIntermediateAngleRadians = FMath::DegreesToRadians(InOutIntermediateAngle);
	float InOutIntermediateAngularVelocityRadians = FMath::DegreesToRadians(InOutIntermediateAngularVelocity);
	SpringMath::CriticalDoubleSpringDamperAngle(InOutAngleRadians, InOutAngularVelocityRadians, InOutIntermediateAngleRadians, InOutIntermediateAngularVelocityRadians, FMath::DegreesToRadians(TargetAngle), SmoothingTime, DeltaTime);
	InOutAngle = FMath::RadiansToDegrees(InOutAngleRadians);
	InOutAngularVelocity = FMath::RadiansToDegrees(InOutAngularVelocityRadians);
	InOutIntermediateAngle = FMath::RadiansToDegrees(InOutIntermediateAngleRadians);
	InOutIntermediateAngularVelocity = FMath::RadiansToDegrees(InOutIntermediateAngularVelocityRadians);
}

void UBlueprintSpringMathLibrary::CriticalDoubleSpringDampQuat(FQuat& InOutRotation, FVector& InOutAngularVelocity, FQuat& InOutIntermediateRotation, FVector& InOutIntermediateAngularVelocity, const FQuat& TargetRotation,
	float DeltaTime, float SmoothingTime)
{
	FVector InOutAngularVelocityRadians = FMath::DegreesToRadians(InOutAngularVelocity);
	FVector InOutIntermediateAngularVelocityRadians = FMath::DegreesToRadians(InOutIntermediateAngularVelocity);
	SpringMath::CriticalDoubleSpringDamperQuat(InOutRotation, InOutAngularVelocityRadians, InOutIntermediateRotation, InOutIntermediateAngularVelocityRadians, TargetRotation, SmoothingTime, DeltaTime);
	InOutAngularVelocity = FMath::RadiansToDegrees(InOutAngularVelocityRadians);
	InOutIntermediateAngularVelocity = FMath::RadiansToDegrees(InOutIntermediateAngularVelocityRadians);
}

void UBlueprintSpringMathLibrary::CriticalDoubleSpringDampRotator(FRotator& InOutRotation, FVector& InOutAngularVelocity, FRotator& InOutIntermediateRotation, FVector& InOutIntermediateAngularVelocity, const FRotator& TargetRotation,
	float DeltaTime, float SmoothingTime)
{
	FQuat InOutRotationQuat = InOutRotation.Quaternion();
	FVector InOutAngularVelocityRadians = FMath::DegreesToRadians(InOutAngularVelocity);
	FQuat InOutIntermediateRotationQuat = InOutIntermediateRotation.Quaternion();
	FVector InOutIntermediateAngularVelocityRadians = FMath::DegreesToRadians(InOutIntermediateAngularVelocity);
	SpringMath::CriticalDoubleSpringDamperQuat(InOutRotationQuat, InOutAngularVelocityRadians, InOutIntermediateRotationQuat, InOutIntermediateAngularVelocityRadians, TargetRotation.Quaternion(), SmoothingTime, DeltaTime);
	InOutRotation = InOutRotationQuat.Rotator();
	InOutAngularVelocity = FMath::RadiansToDegrees(InOutAngularVelocityRadians);
	InOutIntermediateRotation = InOutIntermediateRotationQuat.Rotator();
	InOutIntermediateAngularVelocity = FMath::RadiansToDegrees(InOutIntermediateAngularVelocityRadians);
}

void UBlueprintSpringMathLibrary::CriticalDoubleSpringDampScale(FVector& InOutScale, FVector& InOutScalarVelocity, FVector& InOutIntermediateScale, FVector& InOutIntermediateScalarVelocity, const FVector& TargetScale, float DeltaTime, float SmoothingTime)
{
	SpringMath::CriticalDoubleSpringDamperScale(InOutScale, InOutScalarVelocity, InOutIntermediateScale, InOutIntermediateScalarVelocity, TargetScale, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::CriticalDoubleSpringDampTransform(FTransform& InOutTransform, FSpringTransformVelocity& InOutVelocity, FTransform& InOutIntermediateTransform, FSpringTransformVelocity& InOutIntermediateTransformVelocity, const FTransform& TargetTransform, float DeltaTime, float SmoothingTime)
{
	FVector InOutLocation = InOutTransform.GetLocation();
	FQuat InOutRotation = InOutTransform.GetRotation();
	FVector InOutScale = InOutTransform.GetScale3D();
	FVector InOutIntermediateLocation = InOutIntermediateTransform.GetLocation();
	FQuat InOutIntermediateRotation = InOutIntermediateTransform.GetRotation();
	FVector InOutIntermediateScale = InOutIntermediateTransform.GetScale3D();

	CriticalDoubleSpringDampVector(InOutLocation, InOutVelocity.LinearVelocity, InOutIntermediateLocation, InOutIntermediateTransformVelocity.LinearVelocity, TargetTransform.GetLocation(), DeltaTime, SmoothingTime);
	CriticalDoubleSpringDampQuat(InOutRotation, InOutVelocity.AngularVelocity, InOutIntermediateRotation, InOutIntermediateTransformVelocity.AngularVelocity, TargetTransform.GetRotation(), DeltaTime, SmoothingTime);
	CriticalDoubleSpringDampScale(InOutScale, InOutVelocity.ScalarVelocity, InOutIntermediateScale, InOutIntermediateTransformVelocity.ScalarVelocity, TargetTransform.GetScale3D(), DeltaTime, SmoothingTime);

	InOutTransform = FTransform(InOutRotation, InOutLocation, InOutScale);
	InOutIntermediateTransform = FTransform(InOutIntermediateRotation, InOutIntermediateLocation, InOutIntermediateScale);
}

void UBlueprintSpringMathLibrary::VelocitySpringDampFloat(float& InOutX, float& InOutV, float& InOutVi, float TargetX, float MaxSpeed,
                                                          float DeltaTime, float SmoothingTime)
{
	if (MaxSpeed < 0.0f)
	{
		UE_LOGF(LogBlueprintSpingMathLibrary, Warning, "UBlueprintSpringMathLibrary::VelocitySpringDampFloat TargetSpeed cannot be negative");
		return;
	}

	SpringMath::VelocitySpringDamperF(InOutX, InOutV, InOutVi, TargetX, MaxSpeed, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::VelocitySpringDampVector(FVector& InOutX, FVector& InOutV, FVector& InOutVi, const FVector& TargetX, float MaxSpeed,
	float DeltaTime, float SmoothingTime)
{
	if (MaxSpeed < 0.0f)
	{
		UE_LOGF(LogBlueprintSpingMathLibrary, Warning, "UBlueprintSpringMathLibrary::VelocitySpringDampVector TargetSpeed cannot be negative");
		return;
	}

	SpringMath::VelocitySpringDamper(InOutX, InOutV, InOutVi, TargetX, MaxSpeed, SmoothingTime, DeltaTime);
}

void UBlueprintSpringMathLibrary::VelocitySpringDampVector2D(FVector2D& InOutX, FVector2D& InOutV, FVector2D& InOutVi, const FVector2D& TargetX,
	float MaxSpeed, float DeltaTime, float SmoothingTime)
{
	if (MaxSpeed < 0.0f)
	{
		UE_LOGF(LogBlueprintSpingMathLibrary, Warning, "UBlueprintSpringMathLibrary::VelocitySpringDampVector2D TargetSpeed cannot be negative");
		return;
	}

	SpringMath::VelocitySpringDamper(InOutX, InOutV, InOutVi, TargetX, MaxSpeed, SmoothingTime, DeltaTime);
}

float UBlueprintSpringMathLibrary::DampFloat(float Value, float Target, float DeltaTime, float SmoothingTime)
{
	float DampedValue = Value;
	FMath::ExponentialSmoothingApprox(DampedValue, Target, DeltaTime, SmoothingTime);
	return DampedValue;
}

float UBlueprintSpringMathLibrary::DampAngle(float Angle, float TargetAngle, float DeltaTime, float SmoothingTime)
{
	float DampedAngle = FMath::DegreesToRadians(Angle);
	SpringMath::ExponentialSmoothingApproxAngle(DampedAngle, FMath::DegreesToRadians(TargetAngle), DeltaTime, SmoothingTime);
	return FMath::RadiansToDegrees(DampedAngle);
}

FVector UBlueprintSpringMathLibrary::DampVector(const FVector& Value, const FVector& Target, float DeltaTime, float SmoothingTime)
{
	FVector DampedValue = Value;
	FMath::ExponentialSmoothingApprox(DampedValue, Target, DeltaTime, SmoothingTime);
	return DampedValue;
}

FVector2D UBlueprintSpringMathLibrary::DampVector2D(const FVector2D& Value, const FVector2D& Target, float DeltaTime, float SmoothingTime)
{
	FVector2D DampedValue = Value;
	FMath::ExponentialSmoothingApprox(DampedValue, Target, DeltaTime, SmoothingTime);
	return DampedValue;
}

FQuat UBlueprintSpringMathLibrary::DampQuat(const FQuat& Rotation, const FQuat& TargetRotation, float DeltaTime, float SmoothingTime)
{
	FQuat DampedRotation = Rotation;
	SpringMath::ExponentialSmoothingApproxQuat(DampedRotation, TargetRotation, DeltaTime, SmoothingTime);
	return DampedRotation;
}

FRotator UBlueprintSpringMathLibrary::DampRotator(const FRotator& Rotation, const FRotator& TargetRotation, float DeltaTime, float SmoothingTime)
{
	FQuat DampedRotation = Rotation.Quaternion();
	SpringMath::ExponentialSmoothingApproxQuat(DampedRotation, TargetRotation.Quaternion(), DeltaTime, SmoothingTime);
	return DampedRotation.Rotator();
}

FVector UBlueprintSpringMathLibrary::DampScale(const FVector& Scale, const FVector& TargetScale, float DeltaTime, float SmoothingTime)
{
	FVector DampedScale = Scale;
	SpringMath::ExponentialSmoothingApproxScale(DampedScale, TargetScale, DeltaTime, SmoothingTime);
	return DampedScale;
}

FTransform UBlueprintSpringMathLibrary::DampTransform(const FTransform& Transform, const FTransform& TargetTransform, float DeltaTime, float SmoothingTime)
{
	return FTransform(
		DampQuat(Transform.GetRotation(), TargetTransform.GetRotation(), DeltaTime, SmoothingTime),
		DampVector(Transform.GetLocation(), TargetTransform.GetLocation(), DeltaTime, SmoothingTime),
		DampScale(Transform.GetScale3D(), TargetTransform.GetScale3D(), DeltaTime, SmoothingTime));
}

void UBlueprintSpringMathLibrary::SpringCharacterUpdate(FVector& InOutPosition, FVector& InOutVelocity, FVector& InOutAcceleration,
                                                        const FVector& TargetVelocity, float DeltaTime, float SmoothingTime)
{
	SpringMath::SpringCharacterUpdate(InOutPosition, InOutVelocity, InOutAcceleration, TargetVelocity, SmoothingTime, DeltaTime);
}

float UBlueprintSpringMathLibrary::SpringCharacterStoppingDistance(const float InitialVelocity, const float InitialAcceleration, const float SmoothingTime)
{
	return SpringMath::SpringCharacterStoppingDistance(InitialVelocity, InitialAcceleration, SmoothingTime);
}

float UBlueprintSpringMathLibrary::SpringCharacterStoppingTime(const float InitialVelocity, const float SmoothingTime, const float VelocityThreshold)
{
	return SpringMath::SpringCharacterStoppingTime(InitialVelocity, SmoothingTime, VelocityThreshold);
}

float UBlueprintSpringMathLibrary::SpringCharacterSmoothingTimeFromStoppingDistance(const float InitialVelocity, const float StoppingDistance)
{
	return SpringMath::SpringCharacterSmoothingTimeFromStoppingDistance(InitialVelocity, StoppingDistance);
}

float UBlueprintSpringMathLibrary::SpringCharacterSmoothingTimeFromStoppingTime(const float InitialVelocity, const float StoppingTime, const float VelocityThreshold)
{
	return SpringMath::SpringCharacterSmoothingTimeFromStoppingTime(InitialVelocity, StoppingTime, VelocityThreshold);
}

float UBlueprintSpringMathLibrary::SpringCharacterStartingDistance(const float TargetVelocity, const float SmoothingTime, const float VelocityThreshold)
{
	return SpringMath::SpringCharacterStartingDistance(TargetVelocity, SmoothingTime, VelocityThreshold);
}

float UBlueprintSpringMathLibrary::SpringCharacterStartingTime(const float TargetVelocity, const float SmoothingTime, const float VelocityThreshold)
{
	return SpringMath::SpringCharacterStartingTime(TargetVelocity, SmoothingTime, VelocityThreshold);
}

float UBlueprintSpringMathLibrary::SpringCharacterSmoothingTimeFromStartingDistance(const float TargetVelocity, const float StartingDistance, const float VelocityThreshold)
{
	return SpringMath::SpringCharacterSmoothingTimeFromStartingDistance(TargetVelocity, StartingDistance, VelocityThreshold);
}

float UBlueprintSpringMathLibrary::SpringCharacterSmoothingTimeFromStartingTime(const float TargetVelocity, const float StartingTime, const float VelocityThreshold)
{
	return SpringMath::SpringCharacterSmoothingTimeFromStartingTime(TargetVelocity, StartingTime, VelocityThreshold);
}

float UBlueprintSpringMathLibrary::SpringCharacterTurningTime(const float TargetAngle, const float SmoothingTime, const float AngleThreshold)
{
	// Logic is identical for turns vs starts when it comes to times, so we just forward to the following
	return SpringMath::SpringCharacterStartingTime(TargetAngle, SmoothingTime, AngleThreshold);
}

float UBlueprintSpringMathLibrary::SpringCharacterSmoothingTimeFromTurningTime(const float TargetAngle, const float TurningTime, const float AngleThreshold)
{
	// Logic is identical for turns vs starts when it comes to times, so we just forward to the following
	return SpringMath::SpringCharacterSmoothingTimeFromStartingTime(TargetAngle, TurningTime, AngleThreshold);
}

float UBlueprintSpringMathLibrary::SpringCharacterMaximumAcceleration(const float InitialVelocity, const float InitialAcceleration, const float TargetVelocity, const float SmoothingTime)
{
	return SpringMath::SpringCharacterMaximumAcceleration(InitialVelocity, InitialAcceleration, TargetVelocity, SmoothingTime);
}

float UBlueprintSpringMathLibrary::SpringCharacterSmoothingTimeFromMaximumAcceleration(const float InitialVelocity, const float TargetVelocity, const float MaximumAcceleration)
{
	return SpringMath::SpringCharacterSmoothingTimeFromMaximumAcceleration(InitialVelocity, TargetVelocity, MaximumAcceleration);
}

float UBlueprintSpringMathLibrary::SpringCharacterMaximumAngularVelocity(const float InitialAngle, const float InitialAngularVelocity, const float TargetAngle, const float SmoothingTime)
{
	return FMath::RadiansToDegrees(SpringMath::SpringCharacterMaximumAngularVelocity(FMath::DegreesToRadians(InitialAngle), FMath::DegreesToRadians(InitialAngularVelocity), FMath::DegreesToRadians(TargetAngle), SmoothingTime));
}

float UBlueprintSpringMathLibrary::SpringCharacterSmoothingTimeFromMaximumAngularVelocity(const float InitialAngle, const float TargetAngle, const float MaximumAngularVelocity)
{
	return SpringMath::SpringCharacterSmoothingTimeFromMaximumAngularVelocity(FMath::DegreesToRadians(InitialAngle), FMath::DegreesToRadians(TargetAngle), FMath::DegreesToRadians(MaximumAngularVelocity));
}

void UBlueprintSpringMathLibrary::VelocitySpringCharacterUpdate(FVector& InOutPosition, FVector& InOutVelocity, FVector& InOutVelocityIntermediate,
	FVector& InOutAcceleration, const FVector& TargetVelocity, float DeltaTime, float SmoothingTime, float MaxAcceleration)
{
	SpringMath::VelocitySpringCharacterUpdate(InOutPosition, InOutVelocity, InOutVelocityIntermediate, InOutAcceleration, TargetVelocity, SmoothingTime, MaxAcceleration, DeltaTime);
}

float UBlueprintSpringMathLibrary::ConvertSmoothingTimeToStrength(float SmoothingTime)
{
	return SpringMath::SmoothingTimeToStrength(SmoothingTime);
}

float UBlueprintSpringMathLibrary::ConvertStrengthToSmoothingTime(float Strength)
{
	return SpringMath::StrengthToSmoothingTime(Strength);
}

float UBlueprintSpringMathLibrary::ConvertHalfLifeToSmoothingTime(float HalfLife)
{
	return SpringMath::HalfLifeToSmoothingTime(HalfLife);
}

float UBlueprintSpringMathLibrary::ConvertSmoothingTimeToHalfLife(float SmoothingTime)
{
	return SpringMath::SmoothingTimeToHalfLife(SmoothingTime);
}

void UBlueprintSpringMathLibrary::TrackVelocityFloat(float& InOutValue, float& InOutVelocity, float InValue, float DeltaTime)
{
	SpringMath::TrackVelocity(InOutValue, InOutVelocity, InValue, DeltaTime);
}

void UBlueprintSpringMathLibrary::TrackVelocityVector(FVector& InOutValue, FVector& InOutVelocity, FVector InValue, float DeltaTime)
{
	SpringMath::TrackVelocity(InOutValue, InOutVelocity, InValue, DeltaTime);
}

void UBlueprintSpringMathLibrary::TrackVelocityVector2D(FVector2D& InOutValue, FVector2D& InOutVelocity, FVector2D InValue, float DeltaTime)
{
	SpringMath::TrackVelocity(InOutValue, InOutVelocity, InValue, DeltaTime);
}

void UBlueprintSpringMathLibrary::TrackVelocityQuat(FQuat& InOutValue, FVector& InOutVelocity, FQuat InValue, float DeltaTime)
{
	FVector InOutVelocityRadians = FMath::DegreesToRadians(InOutVelocity);
	SpringMath::TrackVelocityQuat(InOutValue, InOutVelocityRadians, InValue, DeltaTime);
	InOutVelocity = FMath::RadiansToDegrees(InOutVelocityRadians);
}

void UBlueprintSpringMathLibrary::TrackVelocityRotator(FRotator& InOutValue, FVector& InOutVelocity, FRotator InValue, float DeltaTime)
{
	FQuat InOutValueQuat = InOutValue.Quaternion();
	FVector InOutVelocityRadians = FMath::DegreesToRadians(InOutVelocity);
	SpringMath::TrackVelocityQuat(InOutValueQuat, InOutVelocityRadians, InValue.Quaternion(), DeltaTime);
	InOutValue = InOutValueQuat.Rotator();
	InOutVelocity = FMath::RadiansToDegrees(InOutVelocityRadians);
}

void UBlueprintSpringMathLibrary::TrackVelocityAngle(float& InOutValue, float& InOutVelocity, float InValue, float DeltaTime)
{
	float InOutValueRadians = FMath::DegreesToRadians(InOutValue);
	float InOutVelocityRadians = FMath::DegreesToRadians(InOutVelocity);
	SpringMath::TrackVelocityAngle(InOutValueRadians, InOutVelocityRadians, FMath::DegreesToRadians(InValue), DeltaTime);
	InOutValue = FMath::RadiansToDegrees(InOutValueRadians);
	InOutVelocity = FMath::RadiansToDegrees(InOutVelocityRadians);
}

void UBlueprintSpringMathLibrary::TrackVelocityScale(FVector& InOutValue, FVector& InOutVelocity, FVector InValue, float DeltaTime)
{
	SpringMath::TrackVelocityScale(InOutValue, InOutVelocity, InValue, DeltaTime);
}

void UBlueprintSpringMathLibrary::TrackVelocityTransform(FTransform& InOutValue, FSpringTransformVelocity& InOutVelocity, FTransform InValue, float DeltaTime)
{
	FVector InOutLocation = InOutValue.GetLocation();
	FQuat InOutRotation = InOutValue.GetRotation();
	FVector InOutScale = InOutValue.GetScale3D();

	TrackVelocityVector(InOutLocation, InOutVelocity.LinearVelocity, InValue.GetLocation(), DeltaTime);
	TrackVelocityQuat(InOutRotation, InOutVelocity.AngularVelocity, InValue.GetRotation(), DeltaTime);
	TrackVelocityScale(InOutScale, InOutVelocity.ScalarVelocity, InValue.GetScale3D(), DeltaTime);

	InOutValue = FTransform(InOutRotation, InOutLocation, InOutScale);
}

float UBlueprintSpringMathLibrary::SpringInertializeApplyToFloat(float InValue, float InValueOffset, float InVelocityOffset, float TimeSinceTransition, float SmoothingTime)
{
	float ValueVelocity = 0.0f;
	SpringMath::SpringInertializeApply(InValue, ValueVelocity, InValueOffset, InVelocityOffset, TimeSinceTransition, SmoothingTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::SpringInertializeTransitionFloat(float& InOutValueOffset, float& InOutVelocityOffset, float InSrcValue, float InSrcVelocity, float InDstValue, float InDstVelocity, float TimeSinceTransition, float SmoothingTime)
{
	SpringMath::SpringInertializeTransition(InOutValueOffset, InOutVelocityOffset, InSrcValue, InSrcVelocity, InDstValue, InDstVelocity, TimeSinceTransition, SmoothingTime);
}

FVector UBlueprintSpringMathLibrary::SpringInertializeApplyToVector(FVector InValue, FVector InValueOffset, FVector InVelocityOffset, float TimeSinceTransition, float SmoothingTime)
{
	FVector ValueVelocity = FVector::ZeroVector;
	SpringMath::SpringInertializeApply(InValue, ValueVelocity, InValueOffset, InVelocityOffset, TimeSinceTransition, SmoothingTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::SpringInertializeTransitionVector(FVector& InOutValueOffset, FVector& InOutVelocityOffset, FVector InSrcValue, FVector InSrcVelocity, FVector InDstValue, FVector InDstVelocity, float TimeSinceTransition, float SmoothingTime)
{
	SpringMath::SpringInertializeTransition(InOutValueOffset, InOutVelocityOffset, InSrcValue, InSrcVelocity, InDstValue, InDstVelocity, TimeSinceTransition, SmoothingTime);
}

FVector2D UBlueprintSpringMathLibrary::SpringInertializeApplyToVector2D(FVector2D InValue, FVector2D InValueOffset, FVector2D InVelocityOffset, float TimeSinceTransition, float SmoothingTime)
{
	FVector2D ValueVelocity = FVector2D::ZeroVector;
	SpringMath::SpringInertializeApply(InValue, ValueVelocity, InValueOffset, InVelocityOffset, TimeSinceTransition, SmoothingTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::SpringInertializeTransitionVector2D(FVector2D& InOutValueOffset, FVector2D& InOutVelocityOffset, FVector2D InSrcValue, FVector2D InSrcVelocity, FVector2D InDstValue, FVector2D InDstVelocity, float TimeSinceTransition, float SmoothingTime)
{
	SpringMath::SpringInertializeTransition(InOutValueOffset, InOutVelocityOffset, InSrcValue, InSrcVelocity, InDstValue, InDstVelocity, TimeSinceTransition, SmoothingTime);
}

FQuat UBlueprintSpringMathLibrary::SpringInertializeApplyToQuat(FQuat InRotation, FQuat InRotationOffset, FVector InAngularVelocityOffset, float TimeSinceTransition, float SmoothingTime)
{
	FVector ValueVelocity = FVector::ZeroVector;
	SpringMath::SpringInertializeApplyQuat(InRotation, ValueVelocity, InRotationOffset, FMath::DegreesToRadians(InAngularVelocityOffset), TimeSinceTransition, SmoothingTime);
	return InRotation;
}

void UBlueprintSpringMathLibrary::SpringInertializeTransitionQuat(FQuat& InOutRotationOffset, FVector& InOutAngularVelocityOffset, FQuat InSrcRotation, FVector InSrcAngularVelocity, FQuat InDstRotation, FVector InDstAngularVelocity, float TimeSinceTransition, float SmoothingTime)
{
	FVector InOutAngularVelocityOffsetRadians = FMath::DegreesToRadians(InOutAngularVelocityOffset);
	SpringMath::SpringInertializeTransitionQuat(InOutRotationOffset, InOutAngularVelocityOffsetRadians, InSrcRotation, FMath::DegreesToRadians(InSrcAngularVelocity), InDstRotation, FMath::DegreesToRadians(InDstAngularVelocity), TimeSinceTransition, SmoothingTime);
	InOutAngularVelocityOffset = FMath::RadiansToDegrees(InOutAngularVelocityOffsetRadians);
}

FRotator UBlueprintSpringMathLibrary::SpringInertializeApplyToRotator(FRotator InRotation, FRotator InRotationOffset, FVector InAngularVelocityOffset, float TimeSinceTransition, float SmoothingTime)
{
	FQuat InRotationQuat = InRotation.Quaternion();
	FVector ValueVelocity = FVector::ZeroVector;
	SpringMath::SpringInertializeApplyQuat(InRotationQuat, ValueVelocity, InRotationOffset.Quaternion(), FMath::DegreesToRadians(InAngularVelocityOffset), TimeSinceTransition, SmoothingTime);
	return InRotationQuat.Rotator();
}

void UBlueprintSpringMathLibrary::SpringInertializeTransitionRotator(FRotator& InOutRotationOffset, FVector& InOutAngularVelocityOffset, FRotator InSrcRotation, FVector InSrcAngularVelocity, FRotator InDstRotation, FVector InDstAngularVelocity, float TimeSinceTransition, float SmoothingTime)
{
	FQuat InOutRotationOffsetQuat = InOutRotationOffset.Quaternion();
	FVector InOutAngularVelocityOffsetRadians = FMath::DegreesToRadians(InOutAngularVelocityOffset);
	SpringMath::SpringInertializeTransitionQuat(InOutRotationOffsetQuat, InOutAngularVelocityOffsetRadians, InSrcRotation.Quaternion(), FMath::DegreesToRadians(InSrcAngularVelocity), InDstRotation.Quaternion(), FMath::DegreesToRadians(InDstAngularVelocity), TimeSinceTransition, SmoothingTime);
	InOutRotationOffset = InOutRotationOffsetQuat.Rotator();
	InOutAngularVelocityOffset = FMath::RadiansToDegrees(InOutAngularVelocityOffsetRadians);
}

float UBlueprintSpringMathLibrary::SpringInertializeApplyToAngle(float InAngle, float InAngleOffset, float InAngularVelocityOffset, float TimeSinceTransition, float SmoothingTime)
{
	float InAngleRadians = FMath::DegreesToRadians(InAngle);
	float ValueVelocity = 0.0f;
	SpringMath::SpringInertializeApplyAngle(InAngleRadians, ValueVelocity, FMath::DegreesToRadians(InAngleOffset), FMath::DegreesToRadians(InAngularVelocityOffset), TimeSinceTransition, SmoothingTime);
	return FMath::RadiansToDegrees(InAngleRadians);
}

void UBlueprintSpringMathLibrary::SpringInertializeTransitionAngle(float& InOutAngleOffset, float& InOutAngularVelocityOffset, float InSrcAngle, float InSrcAngularVelocity, float InDstAngle, float InDstAngularVelocity, float TimeSinceTransition, float SmoothingTime)
{
	float InOutAngleOffsetRadians = FMath::DegreesToRadians(InOutAngleOffset);
	float InOutAngularVelocityOffsetRadians = FMath::DegreesToRadians(InOutAngularVelocityOffset);
	SpringMath::SpringInertializeTransitionAngle(InOutAngleOffsetRadians, InOutAngularVelocityOffsetRadians, FMath::DegreesToRadians(InSrcAngle), FMath::DegreesToRadians(InSrcAngularVelocity), FMath::DegreesToRadians(InDstAngle), FMath::DegreesToRadians(InDstAngularVelocity), TimeSinceTransition, SmoothingTime);
	InOutAngleOffset = FMath::RadiansToDegrees(InOutAngleOffsetRadians);
	InOutAngularVelocityOffset = FMath::RadiansToDegrees(InOutAngularVelocityOffsetRadians);
}

FVector UBlueprintSpringMathLibrary::SpringInertializeApplyToScale(FVector InScale, FVector InScaleOffset, FVector InScalarVelocityOffset, float TimeSinceTransition, float SmoothingTime)
{
	FVector ValueVelocity = FVector::ZeroVector;
	SpringMath::SpringInertializeApplyScale(InScale, ValueVelocity, InScaleOffset, InScalarVelocityOffset, TimeSinceTransition, SmoothingTime);
	return InScale;
}

void UBlueprintSpringMathLibrary::SpringInertializeTransitionScale(FVector& InOutScaleOffset, FVector& InOutScalarVelocityOffset, FVector InSrcScale, FVector InSrcScalarVelocity, FVector InDstScale, FVector InDstScalarVelocity, float TimeSinceTransition, float SmoothingTime)
{
	SpringMath::SpringInertializeTransitionScale(InOutScaleOffset, InOutScalarVelocityOffset, InSrcScale, InSrcScalarVelocity, InDstScale, InDstScalarVelocity, TimeSinceTransition, SmoothingTime);
}

FTransform UBlueprintSpringMathLibrary::SpringInertializeApplyToTransform(FTransform InTransform, FTransform InTransformOffset, FSpringTransformVelocity InVelocityOffset, float TimeSinceTransition, float SmoothingTime)
{
	return FTransform(
		SpringInertializeApplyToQuat(InTransform.GetRotation(), InTransformOffset.GetRotation(), InVelocityOffset.AngularVelocity, TimeSinceTransition, SmoothingTime),
		SpringInertializeApplyToVector(InTransform.GetLocation(), InTransformOffset.GetLocation(), InVelocityOffset.LinearVelocity, TimeSinceTransition, SmoothingTime),
		SpringInertializeApplyToScale(InTransform.GetScale3D(), InTransformOffset.GetScale3D(), InVelocityOffset.ScalarVelocity, TimeSinceTransition, SmoothingTime));
}

void UBlueprintSpringMathLibrary::SpringInertializeTransitionTransform(FTransform& InOutTransformOffset, FSpringTransformVelocity& InOutVelocityOffset, FTransform InSrcTransform, FSpringTransformVelocity InSrcVelocity, FTransform InDstTransform, FSpringTransformVelocity InDstVelocity, float TimeSinceTransition, float SmoothingTime)
{
	FVector InOutLocation = InOutTransformOffset.GetLocation();
	FQuat InOutRotation = InOutTransformOffset.GetRotation();
	FVector InOutScale = InOutTransformOffset.GetScale3D();

	SpringInertializeTransitionVector(InOutLocation, InOutVelocityOffset.LinearVelocity, InSrcTransform.GetLocation(), InSrcVelocity.LinearVelocity, InDstTransform.GetLocation(), InDstVelocity.LinearVelocity, TimeSinceTransition, SmoothingTime);
	SpringInertializeTransitionQuat(InOutRotation, InOutVelocityOffset.AngularVelocity, InSrcTransform.GetRotation(), InSrcVelocity.AngularVelocity, InDstTransform.GetRotation(), InDstVelocity.AngularVelocity, TimeSinceTransition, SmoothingTime);
	SpringInertializeTransitionScale(InOutScale, InOutVelocityOffset.ScalarVelocity, InSrcTransform.GetScale3D(), InSrcVelocity.ScalarVelocity, InDstTransform.GetScale3D(), InDstVelocity.ScalarVelocity, TimeSinceTransition, SmoothingTime);

	InOutTransformOffset = FTransform(InOutRotation, InOutLocation, InOutScale);
}

float UBlueprintSpringMathLibrary::CubicInertializeApplyToFloat(float InValue, float InValueOffset, float InVelocityOffset, float TimeSinceTransition, float BlendTime)
{
	float ValueVelocity = 0.0f;
	SpringMath::CubicInertializeApply(InValue, ValueVelocity, InValueOffset, InVelocityOffset, TimeSinceTransition, BlendTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::CubicInertializeTransitionFloat(float& InOutValueOffset, float& InOutVelocityOffset, float InSrcValue, float InSrcVelocity, float InDstValue, float InDstVelocity, float TimeSinceTransition, float BlendTime)
{
	SpringMath::CubicInertializeTransition(InOutValueOffset, InOutVelocityOffset, InSrcValue, InSrcVelocity, InDstValue, InDstVelocity, TimeSinceTransition, BlendTime);
}

FVector UBlueprintSpringMathLibrary::CubicInertializeApplyToVector(FVector InValue, FVector InValueOffset, FVector InVelocityOffset, float TimeSinceTransition, float BlendTime)
{
	FVector ValueVelocity = FVector::ZeroVector;
	SpringMath::CubicInertializeApply(InValue, ValueVelocity, InValueOffset, InVelocityOffset, TimeSinceTransition, BlendTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::CubicInertializeTransitionVector(FVector& InOutValueOffset, FVector& InOutVelocityOffset, FVector InSrcValue, FVector InSrcVelocity, FVector InDstValue, FVector InDstVelocity, float TimeSinceTransition, float BlendTime)
{
	SpringMath::CubicInertializeTransition(InOutValueOffset, InOutVelocityOffset, InSrcValue, InSrcVelocity, InDstValue, InDstVelocity, TimeSinceTransition, BlendTime);
}

FVector2D UBlueprintSpringMathLibrary::CubicInertializeApplyToVector2D(FVector2D InValue, FVector2D InValueOffset, FVector2D InVelocityOffset, float TimeSinceTransition, float BlendTime)
{
	FVector2D ValueVelocity = FVector2D::ZeroVector;
	SpringMath::CubicInertializeApply(InValue, ValueVelocity, InValueOffset, InVelocityOffset, TimeSinceTransition, BlendTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::CubicInertializeTransitionVector2D(FVector2D& InOutValueOffset, FVector2D& InOutVelocityOffset, FVector2D InSrcValue, FVector2D InSrcVelocity, FVector2D InDstValue, FVector2D InDstVelocity, float TimeSinceTransition, float BlendTime)
{
	SpringMath::CubicInertializeTransition(InOutValueOffset, InOutVelocityOffset, InSrcValue, InSrcVelocity, InDstValue, InDstVelocity, TimeSinceTransition, BlendTime);
}

FQuat UBlueprintSpringMathLibrary::CubicInertializeApplyToQuat(FQuat InRotation, FQuat InRotationOffset, FVector InAngularVelocityOffset, float TimeSinceTransition, float BlendTime)
{
	FVector ValueVelocity = FVector::ZeroVector;
	SpringMath::CubicInertializeApplyQuat(InRotation, ValueVelocity, InRotationOffset, FMath::DegreesToRadians(InAngularVelocityOffset), TimeSinceTransition, BlendTime);
	return InRotation;
}

void UBlueprintSpringMathLibrary::CubicInertializeTransitionQuat(FQuat& InOutRotationOffset, FVector& InOutAngularVelocityOffset, FQuat InSrcRotation, FVector InSrcAngularVelocity, FQuat InDstRotation, FVector InDstAngularVelocity, float TimeSinceTransition, float BlendTime)
{
	FVector InOutAngularVelocityOffsetRadians = FMath::DegreesToRadians(InOutAngularVelocityOffset);
	SpringMath::CubicInertializeTransitionQuat(InOutRotationOffset, InOutAngularVelocityOffsetRadians, InSrcRotation, FMath::DegreesToRadians(InSrcAngularVelocity), InDstRotation, FMath::DegreesToRadians(InDstAngularVelocity), TimeSinceTransition, BlendTime);
	InOutAngularVelocityOffset = FMath::RadiansToDegrees(InOutAngularVelocityOffsetRadians);
}

FRotator UBlueprintSpringMathLibrary::CubicInertializeApplyToRotator(FRotator InRotation, FRotator InRotationOffset, FVector InAngularVelocityOffset, float TimeSinceTransition, float BlendTime)
{
	FQuat InRotationQuat = InRotation.Quaternion();
	FVector ValueVelocity = FVector::ZeroVector;
	SpringMath::CubicInertializeApplyQuat(InRotationQuat, ValueVelocity, InRotationOffset.Quaternion(), FMath::DegreesToRadians(InAngularVelocityOffset), TimeSinceTransition, BlendTime);
	return InRotationQuat.Rotator();
}

void UBlueprintSpringMathLibrary::CubicInertializeTransitionRotator(FRotator& InOutRotationOffset, FVector& InOutAngularVelocityOffset, FRotator InSrcRotation, FVector InSrcAngularVelocity, FRotator InDstRotation, FVector InDstAngularVelocity, float TimeSinceTransition, float BlendTime)
{
	FQuat InOutRotationOffsetQuat = InOutRotationOffset.Quaternion();
	FVector InOutAngularVelocityOffsetRadians = FMath::DegreesToRadians(InOutAngularVelocityOffset);
	SpringMath::CubicInertializeTransitionQuat(InOutRotationOffsetQuat, InOutAngularVelocityOffsetRadians, InSrcRotation.Quaternion(), FMath::DegreesToRadians(InSrcAngularVelocity), InDstRotation.Quaternion(), FMath::DegreesToRadians(InDstAngularVelocity), TimeSinceTransition, BlendTime);
	InOutRotationOffset = InOutRotationOffsetQuat.Rotator();
	InOutAngularVelocityOffset = FMath::RadiansToDegrees(InOutAngularVelocityOffsetRadians);
}

float UBlueprintSpringMathLibrary::CubicInertializeApplyToAngle(float InAngle, float InAngleOffset, float InAngularVelocityOffset, float TimeSinceTransition, float BlendTime)
{
	float InAngleRadians = FMath::DegreesToRadians(InAngle);
	float ValueVelocity = 0.0f;
	SpringMath::CubicInertializeApplyAngle(InAngleRadians, ValueVelocity, FMath::DegreesToRadians(InAngleOffset), FMath::DegreesToRadians(InAngularVelocityOffset), TimeSinceTransition, BlendTime);
	return FMath::RadiansToDegrees(InAngleRadians);
}

void UBlueprintSpringMathLibrary::CubicInertializeTransitionAngle(float& InOutAngleOffset, float& InOutAngularVelocityOffset, float InSrcAngle, float InSrcAngularVelocity, float InDstAngle, float InDstAngularVelocity, float TimeSinceTransition, float BlendTime)
{
	float InOutAngleOffsetRadians = FMath::DegreesToRadians(InOutAngleOffset);
	float InOutAngularVelocityOffsetRadians = FMath::DegreesToRadians(InOutAngularVelocityOffset);
	SpringMath::CubicInertializeTransitionAngle(InOutAngleOffsetRadians, InOutAngularVelocityOffsetRadians, FMath::DegreesToRadians(InSrcAngle), FMath::DegreesToRadians(InSrcAngularVelocity), FMath::DegreesToRadians(InDstAngle), FMath::DegreesToRadians(InDstAngularVelocity), TimeSinceTransition, BlendTime);
	InOutAngleOffset = FMath::RadiansToDegrees(InOutAngleOffsetRadians);
	InOutAngularVelocityOffset = FMath::RadiansToDegrees(InOutAngularVelocityOffsetRadians);
}

FVector UBlueprintSpringMathLibrary::CubicInertializeApplyToScale(FVector InScale, FVector InScaleOffset, FVector InScalarVelocityOffset, float TimeSinceTransition, float BlendTime)
{
	FVector ValueVelocity = FVector::ZeroVector;
	SpringMath::CubicInertializeApplyScale(InScale, ValueVelocity, InScaleOffset, InScalarVelocityOffset, TimeSinceTransition, BlendTime);
	return InScale;
}

void UBlueprintSpringMathLibrary::CubicInertializeTransitionScale(FVector& InOutScaleOffset, FVector& InOutScalarVelocityOffset, FVector InSrcScale, FVector InSrcScalarVelocity, FVector InDstScale, FVector InDstScalarVelocity, float TimeSinceTransition, float BlendTime)
{
	SpringMath::CubicInertializeTransitionScale(InOutScaleOffset, InOutScalarVelocityOffset, InSrcScale, InSrcScalarVelocity, InDstScale, InDstScalarVelocity, TimeSinceTransition, BlendTime);
}

FTransform UBlueprintSpringMathLibrary::CubicInertializeApplyToTransform(FTransform InTransform, FTransform InTransformOffset, FSpringTransformVelocity InVelocityOffset, float TimeSinceTransition, float BlendTime)
{
	return FTransform(
		CubicInertializeApplyToQuat(InTransform.GetRotation(), InTransformOffset.GetRotation(), InVelocityOffset.AngularVelocity, TimeSinceTransition, BlendTime),
		CubicInertializeApplyToVector(InTransform.GetLocation(), InTransformOffset.GetLocation(), InVelocityOffset.LinearVelocity, TimeSinceTransition, BlendTime),
		CubicInertializeApplyToScale(InTransform.GetScale3D(), InTransformOffset.GetScale3D(), InVelocityOffset.ScalarVelocity, TimeSinceTransition, BlendTime));
}

void UBlueprintSpringMathLibrary::CubicInertializeTransitionTransform(FTransform& InOutTransformOffset, FSpringTransformVelocity& InOutVelocityOffset, FTransform InSrcTransform, FSpringTransformVelocity InSrcVelocity, FTransform InDstTransform, FSpringTransformVelocity InDstVelocity, float TimeSinceTransition, float BlendTime)
{
	FVector InOutLocation = InOutTransformOffset.GetLocation();
	FQuat InOutRotation = InOutTransformOffset.GetRotation();
	FVector InOutScale = InOutTransformOffset.GetScale3D();

	CubicInertializeTransitionVector(InOutLocation, InOutVelocityOffset.LinearVelocity, InSrcTransform.GetLocation(), InSrcVelocity.LinearVelocity, InDstTransform.GetLocation(), InDstVelocity.LinearVelocity, TimeSinceTransition, BlendTime);
	CubicInertializeTransitionQuat(InOutRotation, InOutVelocityOffset.AngularVelocity, InSrcTransform.GetRotation(), InSrcVelocity.AngularVelocity, InDstTransform.GetRotation(), InDstVelocity.AngularVelocity, TimeSinceTransition, BlendTime);
	CubicInertializeTransitionScale(InOutScale, InOutVelocityOffset.ScalarVelocity, InSrcTransform.GetScale3D(), InSrcVelocity.ScalarVelocity, InDstTransform.GetScale3D(), InDstVelocity.ScalarVelocity, TimeSinceTransition, BlendTime);

	InOutTransformOffset = FTransform(InOutRotation, InOutLocation, InOutScale);
}


float UBlueprintSpringMathLibrary::DeadBlendApplyToFloat(float InValue, float InValueTransition, float InVelocityTransition, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	float VelocityValue = 0.0f;
	SpringMath::DeadBlendApply(InValue, VelocityValue, InValueTransition, InVelocityTransition, TimeSinceTransition, BlendTime, SmoothingTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::DeadBlendTransitionFloat(float& InOutValueTransition, float& InOutVelocityTransition, float InSrcValue, float InSrcVelocity, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	SpringMath::DeadBlendTransition(InOutValueTransition, InOutVelocityTransition, InSrcValue, InSrcVelocity, TimeSinceTransition, BlendTime, SmoothingTime);
}

FVector UBlueprintSpringMathLibrary::DeadBlendApplyToVector(FVector InValue, FVector InValueTransition, FVector InVelocityTransition, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	FVector VelocityValue = FVector::ZeroVector;
	SpringMath::DeadBlendApply(InValue, VelocityValue, InValueTransition, InVelocityTransition, TimeSinceTransition, BlendTime, SmoothingTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::DeadBlendTransitionVector(FVector& InOutValueTransition, FVector& InOutVelocityTransition, FVector InSrcValue, FVector InSrcVelocity, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	SpringMath::DeadBlendTransition(InOutValueTransition, InOutVelocityTransition, InSrcValue, InSrcVelocity, TimeSinceTransition, BlendTime, SmoothingTime);
}

FVector2D UBlueprintSpringMathLibrary::DeadBlendApplyToVector2D(FVector2D InValue, FVector2D InValueTransition, FVector2D InVelocityTransition, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	FVector2D VelocityValue = FVector2D::ZeroVector;
	SpringMath::DeadBlendApply(InValue, VelocityValue, InValueTransition, InVelocityTransition, TimeSinceTransition, BlendTime, SmoothingTime);
	return InValue;
}

void UBlueprintSpringMathLibrary::DeadBlendTransitionVector2D(FVector2D& InOutValueTransition, FVector2D& InOutVelocityTransition, FVector2D InSrcValue, FVector2D InSrcVelocity, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	SpringMath::DeadBlendTransition(InOutValueTransition, InOutVelocityTransition, InSrcValue, InSrcVelocity, TimeSinceTransition, BlendTime, SmoothingTime);
}

FQuat UBlueprintSpringMathLibrary::DeadBlendApplyToQuat(FQuat InRotation, FQuat InRotationTransition, FVector InAngularVelocityTransition, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	FVector VelocityValue = FVector::ZeroVector;
	SpringMath::DeadBlendApplyQuat(InRotation, VelocityValue, InRotationTransition, FMath::DegreesToRadians(InAngularVelocityTransition), TimeSinceTransition, BlendTime, SmoothingTime);
	return InRotation;
}

void UBlueprintSpringMathLibrary::DeadBlendTransitionQuat(FQuat& InOutRotationTransition, FVector& InOutAngularVelocityTransition, FQuat InSrcRotation, FVector InSrcAngularVelocity, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	FVector InOutAngularVelocityTransitionRadians = FMath::DegreesToRadians(InOutAngularVelocityTransition);
	SpringMath::DeadBlendTransitionQuat(InOutRotationTransition, InOutAngularVelocityTransitionRadians, InSrcRotation, FMath::DegreesToRadians(InSrcAngularVelocity), TimeSinceTransition, BlendTime, SmoothingTime);
	InOutAngularVelocityTransition = FMath::RadiansToDegrees(InOutAngularVelocityTransitionRadians);
}

FRotator UBlueprintSpringMathLibrary::DeadBlendApplyToRotator(FRotator InRotation, FRotator InRotationTransition, FVector InAngularVelocityTransition, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	FVector VelocityValue = FVector::ZeroVector;
	FQuat InRotationQuat = InRotation.Quaternion();
	SpringMath::DeadBlendApplyQuat(InRotationQuat, VelocityValue, InRotationTransition.Quaternion(), FMath::DegreesToRadians(InAngularVelocityTransition), TimeSinceTransition, BlendTime, SmoothingTime);
	return InRotationQuat.Rotator();
}

void UBlueprintSpringMathLibrary::DeadBlendTransitionRotator(FRotator& InOutRotationTransition, FVector& InOutAngularVelocityTransition, FRotator InSrcRotation, FVector InSrcAngularVelocity, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	FQuat InOutRotationTransitionQuat = InOutRotationTransition.Quaternion();
	FVector InOutAngularVelocityTransitionRadians = FMath::DegreesToRadians(InOutAngularVelocityTransition);
	SpringMath::DeadBlendTransitionQuat(InOutRotationTransitionQuat, InOutAngularVelocityTransitionRadians, InSrcRotation.Quaternion(), FMath::DegreesToRadians(InSrcAngularVelocity), TimeSinceTransition, BlendTime, SmoothingTime);
	InOutRotationTransition = InOutRotationTransitionQuat.Rotator();
	InOutAngularVelocityTransition = FMath::RadiansToDegrees(InOutAngularVelocityTransitionRadians);
}

float UBlueprintSpringMathLibrary::DeadBlendApplyToAngle(float InAngle, float InAngleTransition, float InAngularVelocityTransition, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	float InAngleRadians = FMath::DegreesToRadians(InAngle);
	float VelocityValue = 0.0f;
	SpringMath::DeadBlendApplyAngle(InAngleRadians, VelocityValue, FMath::DegreesToRadians(InAngleTransition), FMath::DegreesToRadians(InAngularVelocityTransition), TimeSinceTransition, BlendTime, SmoothingTime);
	return FMath::RadiansToDegrees(InAngleRadians);
}

void UBlueprintSpringMathLibrary::DeadBlendTransitionAngle(float& InOutAngleTransition, float& InOutAngularVelocityTransition, float InSrcAngle, float InSrcAngularVelocity, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	float InOutAngleTransitionRadians = FMath::DegreesToRadians(InOutAngleTransition);
	float InOutAngularVelocityTransitionRadians = FMath::DegreesToRadians(InOutAngularVelocityTransition);
	SpringMath::DeadBlendTransitionAngle(InOutAngleTransitionRadians, InOutAngularVelocityTransitionRadians, FMath::DegreesToRadians(InSrcAngle), FMath::DegreesToRadians(InSrcAngularVelocity), TimeSinceTransition, BlendTime, SmoothingTime);
	InOutAngleTransition = FMath::RadiansToDegrees(InOutAngleTransitionRadians);
	InOutAngularVelocityTransition = FMath::RadiansToDegrees(InOutAngularVelocityTransitionRadians);
}

FVector UBlueprintSpringMathLibrary::DeadBlendApplyToScale(FVector InScale, FVector InScaleTransition, FVector InScalarVelocityTransition, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	FVector VelocityValue = FVector::ZeroVector;
	SpringMath::DeadBlendApplyScale(InScale, VelocityValue, InScaleTransition, InScalarVelocityTransition, TimeSinceTransition, BlendTime, SmoothingTime);
	return InScale;
}

void UBlueprintSpringMathLibrary::DeadBlendTransitionScale(FVector& InOutScaleTransition, FVector& InOutScalarVelocityTransition, FVector InSrcScale, FVector InSrcScalarVelocity, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	SpringMath::DeadBlendTransitionScale(InOutScaleTransition, InOutScalarVelocityTransition, InSrcScale, InSrcScalarVelocity, TimeSinceTransition, BlendTime, SmoothingTime);
}

FTransform UBlueprintSpringMathLibrary::DeadBlendApplyToTransform(FTransform InTransform, FTransform InTransformTransition, FSpringTransformVelocity InVelocityTransition, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	return FTransform(
		DeadBlendApplyToQuat(InTransform.GetRotation(), InTransformTransition.GetRotation(), InVelocityTransition.AngularVelocity, TimeSinceTransition, BlendTime, SmoothingTime),
		DeadBlendApplyToVector(InTransform.GetLocation(), InTransformTransition.GetLocation(), InVelocityTransition.LinearVelocity, TimeSinceTransition, BlendTime, SmoothingTime),
		DeadBlendApplyToScale(InTransform.GetScale3D(), InTransformTransition.GetScale3D(), InVelocityTransition.ScalarVelocity, TimeSinceTransition, BlendTime, SmoothingTime));
}

void UBlueprintSpringMathLibrary::DeadBlendTransitionTransform(FTransform& InOutTransformTransition, FSpringTransformVelocity& InOutVelocityTransition, FTransform InSrcTransform, FSpringTransformVelocity InSrcVelocity, float TimeSinceTransition, float BlendTime, float SmoothingTime)
{
	FVector InOutLocation = InOutTransformTransition.GetLocation();
	FQuat InOutRotation = InOutTransformTransition.GetRotation();
	FVector InOutScale = InOutTransformTransition.GetScale3D();

	DeadBlendTransitionVector(InOutLocation, InOutVelocityTransition.LinearVelocity, InSrcTransform.GetLocation(), InSrcVelocity.LinearVelocity, TimeSinceTransition, BlendTime, SmoothingTime);
	DeadBlendTransitionQuat(InOutRotation, InOutVelocityTransition.AngularVelocity, InSrcTransform.GetRotation(), InSrcVelocity.AngularVelocity, TimeSinceTransition, BlendTime, SmoothingTime);
	DeadBlendTransitionScale(InOutScale, InOutVelocityTransition.ScalarVelocity, InSrcTransform.GetScale3D(), InSrcVelocity.ScalarVelocity, TimeSinceTransition, BlendTime, SmoothingTime);

	InOutTransformTransition = FTransform(InOutRotation, InOutLocation, InOutScale);
}