// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneContainer.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/SkeletalMesh.h"
#include "EngineLogs.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/SkeletonRemapping.h"
#include "AnimationRuntime.h"

LLM_DEFINE_TAG(BoneContainer);

DEFINE_LOG_CATEGORY(LogSkeletalControl);

//////////////////////////////////////////////////////////////////////////
// FBoneContainer

FBoneContainer::FBoneContainer()
: Asset(nullptr)
, AssetSkeletalMesh(nullptr)
, AssetSkeleton(nullptr)
, RefSkeleton(nullptr)
, SerialNumber(0)
#if DO_CHECK
, CalculatedForLOD(INDEX_NONE)
#endif
, bDisableRetargeting(false)
, bUseRAWData(false)
, bUseSourceData(false)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));
	BoneIndicesArray.Empty();
	BoneSwitchArray.Empty();
	SkeletonToPoseBoneIndexArray.Empty();
	PoseToSkeletonBoneIndexArray.Empty();
}

FBoneContainer::FBoneContainer(const TArrayView<const FBoneIndexType>& InRequiredBoneIndexArray, const FCurveEvaluationOption& CurveEvalOption, UObject& InAsset)
: BoneIndicesArray(InRequiredBoneIndexArray)
, Asset(&InAsset)
, AssetSkeletalMesh(nullptr)
, AssetSkeleton(nullptr)
, RefSkeleton(nullptr)
, SerialNumber(0)
#if DO_CHECK
, CalculatedForLOD(INDEX_NONE)
#endif
, bDisableRetargeting(false)
, bUseRAWData(false)
, bUseSourceData(false)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));

	const UE::Anim::FCurveFilterSettings CurveFilterSettings(CurveEvalOption.bAllowCurveEvaluation ? UE::Anim::ECurveFilterMode::DisallowFiltered : UE::Anim::ECurveFilterMode::DisallowAll, CurveEvalOption.DisallowedList, CurveEvalOption.LODIndex);
	Initialize(CurveFilterSettings);
}

FBoneContainer::FBoneContainer(const TArrayView<const FBoneIndexType>& InRequiredBoneIndexArray, const UE::Anim::FCurveFilterSettings& InCurveFilterSettings, UObject& InAsset)
	: BoneIndicesArray(InRequiredBoneIndexArray)
	, Asset(&InAsset)
	, AssetSkeletalMesh(nullptr)
	, AssetSkeleton(nullptr)
	, RefSkeleton(nullptr)
	, SerialNumber(0)
#if DO_CHECK
	, CalculatedForLOD(INDEX_NONE)
#endif
	, bDisableRetargeting(false)
	, bUseRAWData(false)
	, bUseSourceData(false)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));
	Initialize(InCurveFilterSettings);
}

void FBoneContainer::InitializeTo(const TArrayView<const FBoneIndexType>& InRequiredBoneIndexArray, const FCurveEvaluationOption& CurveEvalOption, const UObject& InAsset)
{
	const UE::Anim::FCurveFilterSettings CurveFilterSettings(CurveEvalOption.bAllowCurveEvaluation ? UE::Anim::ECurveFilterMode::DisallowFiltered : UE::Anim::ECurveFilterMode::DisallowAll, CurveEvalOption.DisallowedList, CurveEvalOption.LODIndex);
	InitializeTo(InRequiredBoneIndexArray, CurveFilterSettings, InAsset);
}

void FBoneContainer::InitializeTo(const TArrayView<const FBoneIndexType>& InRequiredBoneIndexArray, const UE::Anim::FCurveFilterSettings& CurveFilterSettings, const UObject& InAsset)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));
	BoneIndicesArray = InRequiredBoneIndexArray;
	Asset = const_cast<UObject*>(&InAsset);

	Initialize(CurveFilterSettings);
}

void FBoneContainer::Reset()
{
	Asset = nullptr;
	AssetSkeletalMesh = nullptr;
	AssetSkeleton = nullptr;
	RefSkeleton = nullptr;
	RefPoseOverride = nullptr;

#if DO_CHECK
	CalculatedForLOD = INDEX_NONE;
#endif

	bDisableRetargeting = false;
	bUseRAWData = false;
	bUseSourceData = false;

	BoneIndicesArray.Empty(0);
	BoneSwitchArray.Empty(0);
	SkeletonToPoseBoneIndexArray.Empty(0);
	PoseToSkeletonBoneIndexArray.Empty(0);
	CompactPoseToSkeletonIndex.Empty(0);
	SkeletonToCompactPose.Empty(0);
	CompactPoseParentBones.Empty(0);
	VirtualBoneCompactPoseData.Empty(0);
	CurveFilter = UE::Anim::FCurveFilter();
	CurveFlags = UE::Anim::FBulkCurveFlags();

	// Container changed and is no longer valid, preserve but update our serial number
	RegenerateSerialNumber();
}

struct FBoneContainerScratchArea : public TThreadSingleton<FBoneContainerScratchArea>
{
	TArray<int32> MeshIndexToCompactPoseIndex;
};

void FBoneContainer::Initialize(const UE::Anim::FCurveFilterSettings& InCurveFilterSettings)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));
	RefSkeleton = nullptr;

	UObject* AssetObj =
#if WITH_EDITOR
		Asset.GetEvenIfUnreachable();
#else
		Asset.Get();
#endif
	
	USkeletalMesh* AssetSkeletalMeshObj = Cast<USkeletalMesh>(AssetObj);
	USkeleton* AssetSkeletonObj = nullptr;

#if DO_CHECK
	CalculatedForLOD = InCurveFilterSettings.LODIndex;
#endif
	
	if (AssetSkeletalMeshObj)
	{
		RefSkeleton = &AssetSkeletalMeshObj->GetRefSkeleton();
		AssetSkeletonObj = AssetSkeletalMeshObj->GetSkeleton();
	}
	else
	{
		AssetSkeletonObj = Cast<USkeleton>(AssetObj);
		if (AssetSkeletonObj)
		{
			RefSkeleton = &AssetSkeletonObj->GetReferenceSkeleton();
		}
	}

	// Only supports SkeletalMeshes or Skeletons.
	check( AssetSkeletalMeshObj || AssetSkeletonObj );
	// Skeleton should always be there.
	checkf( AssetSkeletonObj, TEXT("%s missing skeleton"), *GetNameSafe(AssetSkeletalMeshObj));
	check( RefSkeleton );

	AssetSkeleton = AssetSkeletonObj;
	AssetSkeletalMesh = AssetSkeletalMeshObj;

	// Take biggest amount of bones between SkeletalMesh and Skeleton for BoneSwitchArray.
	// SkeletalMesh can have less, but AnimSequences tracks will map to Skeleton which can have more.
	const int32 MaxBones = AssetSkeletonObj ? FMath::Max<int32>(RefSkeleton->GetNum(), AssetSkeletonObj->GetReferenceSkeleton().GetNum()) : RefSkeleton->GetNum();

	// Initialize BoneSwitchArray.
	BoneSwitchArray.Init(false, MaxBones);
	const int32 NumRequiredBones = BoneIndicesArray.Num();
	for(int32 Index=0; Index<NumRequiredBones; Index++)
	{
		const FBoneIndexType BoneIndex = BoneIndicesArray[Index];
		checkSlow( BoneIndex < MaxBones );
		BoneSwitchArray[BoneIndex] = true;
	}

	// Clear remapping table
	SkeletonToPoseBoneIndexArray.Reset();

	// Cache our mapping tables
	// Here we create look up tables between our target asset and its USkeleton's refpose.
	// Most times our Target is a SkeletalMesh
	if (AssetSkeletalMeshObj)
	{
		RemapFromSkelMesh(*AssetSkeletalMeshObj, *AssetSkeletonObj);
	}
	// But we also support a Skeleton's RefPose.
	else
	{
		// Right now we only support a single Skeleton. Skeleton hierarchy coming soon!
		RemapFromSkeleton(*AssetSkeletonObj);
	}

	//Set up compact pose data
	int32 NumReqBones = BoneIndicesArray.Num();
	CompactPoseParentBones.Reset(NumReqBones);

	CompactPoseToSkeletonIndex.Reset(NumReqBones);
	CompactPoseToSkeletonIndex.AddUninitialized(NumReqBones);

	SkeletonToCompactPose.Reset(SkeletonToPoseBoneIndexArray.Num());

	VirtualBoneCompactPoseData.Reset(RefSkeleton->GetVirtualBoneRefData().Num());

	TArray<int32>& MeshIndexToCompactPoseIndex = FBoneContainerScratchArea::Get().MeshIndexToCompactPoseIndex;
	MeshIndexToCompactPoseIndex.Reset(PoseToSkeletonBoneIndexArray.Num());
	MeshIndexToCompactPoseIndex.AddUninitialized(PoseToSkeletonBoneIndexArray.Num());

	for (int32& Item : MeshIndexToCompactPoseIndex)
	{
		Item = -1;
	}
		
	for (int32 CompactBoneIndex = 0; CompactBoneIndex < NumReqBones; ++CompactBoneIndex)
	{
		FBoneIndexType MeshPoseIndex = BoneIndicesArray[CompactBoneIndex];
		MeshIndexToCompactPoseIndex[MeshPoseIndex] = CompactBoneIndex;

		//Parent Bone
		const int32 ParentIndex = GetParentBoneIndex(MeshPoseIndex);
		const int32 CompactParentIndex = ParentIndex == INDEX_NONE ? INDEX_NONE : MeshIndexToCompactPoseIndex[ParentIndex];

		CompactPoseParentBones.Add(FCompactPoseBoneIndex(CompactParentIndex));
	}

	for (int32 CompactBoneIndex = 0; CompactBoneIndex < NumReqBones; ++CompactBoneIndex)
	{
		FBoneIndexType MeshPoseIndex = BoneIndicesArray[CompactBoneIndex];
		CompactPoseToSkeletonIndex[CompactBoneIndex] = PoseToSkeletonBoneIndexArray[MeshPoseIndex];
	}


	for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < SkeletonToPoseBoneIndexArray.Num(); ++SkeletonBoneIndex)
	{
		int32 PoseBoneIndex = SkeletonToPoseBoneIndexArray[SkeletonBoneIndex];
		int32 CompactIndex = (PoseBoneIndex != INDEX_NONE) ? MeshIndexToCompactPoseIndex[PoseBoneIndex] : INDEX_NONE;
		SkeletonToCompactPose.Add(FCompactPoseBoneIndex(CompactIndex));
	}

	for (const FVirtualBoneRefData& VBRefBone : RefSkeleton->GetVirtualBoneRefData())
	{
		const int32 VBInd = MeshIndexToCompactPoseIndex[VBRefBone.VBRefSkelIndex];
		const int32 SourceInd = MeshIndexToCompactPoseIndex[VBRefBone.SourceRefSkelIndex];
		const int32 TargetInd = MeshIndexToCompactPoseIndex[VBRefBone.TargetRefSkelIndex];

		if ((VBInd != INDEX_NONE) && (SourceInd != INDEX_NONE) && (TargetInd != INDEX_NONE))
		{
			VirtualBoneCompactPoseData.Add(FVirtualBoneCompactPoseData(FCompactPoseBoneIndex(VBInd), FCompactPoseBoneIndex(SourceInd), FCompactPoseBoneIndex(TargetInd)));
		}
	}
	// cache required curves list according to new bone sets
	CacheRequiredAnimCurves(InCurveFilterSettings);

	// Reset retargeting cached data look up table.
	RetargetSourceCachedDataLUT.Reset();

	RegenerateSerialNumber();
}

void FBoneContainer::CacheRequiredAnimCurves(const UE::Anim::FCurveFilterSettings& InCurveFilterSettings)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));

	CurveFilter.Empty();
	CurveFilter.SetFilterMode(InCurveFilterSettings.FilterMode);

	CurveFlags.Empty();

	if (USkeleton* Skeleton = AssetSkeleton.GetEvenIfUnreachable())
	{
		// Copy filter curves.
		TArray<FName, TInlineAllocator<32>> FilterCurves;
		if(InCurveFilterSettings.FilterCurves)
		{
			FilterCurves = *InCurveFilterSettings.FilterCurves;
		}
		
		if (InCurveFilterSettings.FilterMode != UE::Anim::ECurveFilterMode::DisallowAll)
		{
			// Apply curve metadata, LOD and linked bones filtering
			Skeleton->ForEachCurveMetaData([this, &InCurveFilterSettings, &FilterCurves](const FName& InCurveName, const FCurveMetaData& InMetaData)
			{
				bool bBeingUsed = true;
				UE::Anim::ECurveElementFlags Flags = UE::Anim::ECurveElementFlags::None;
				UE::Anim::ECurveFilterFlags FilterFlags = UE::Anim::ECurveFilterFlags::None;
				if(InMetaData.Type.bMaterial)
				{
					Flags |= UE::Anim::ECurveElementFlags::Material;
				}

				if(InMetaData.Type.bMorphtarget)
				{
					Flags |= UE::Anim::ECurveElementFlags::MorphTarget;
				}

				const int32 Index = FilterCurves.IndexOfByKey(InCurveName);
				if(Index != INDEX_NONE)
				{
					FilterFlags |= UE::Anim::ECurveFilterFlags::Filtered;
					FilterCurves.RemoveAtSwap(Index, EAllowShrinking::No);
				}
				
				if (InMetaData.MaxLOD < InCurveFilterSettings.LODIndex)
				{
					bBeingUsed = false;
				}
				else if (InMetaData.LinkedBones.Num() > 0)
				{
					bBeingUsed = false;
					for (const FBoneReference& LinkedBoneReference : InMetaData.LinkedBones)
					{
						// when you enter first time, sometimes it does not have all info yet
						if (LinkedBoneReference.BoneIndex != INDEX_NONE && LinkedBoneReference.BoneName != NAME_None)
						{
							// this linked bone always use skeleton index
							ensure(LinkedBoneReference.bUseSkeletonIndex);
							// we want to make sure all the joints are removed from RequiredBones before removing this curve
							if (GetCompactPoseIndexFromSkeletonIndex(LinkedBoneReference.BoneIndex) != INDEX_NONE)
							{
								// still has some joint that matters, do not remove
								bBeingUsed = true;
								break;
							}
						}
					}
				}

				if (!bBeingUsed)
				{
					FilterFlags |= UE::Anim::ECurveFilterFlags::Disallowed;
				}

				if (FilterFlags != UE::Anim::ECurveFilterFlags::None)
				{
					// Add curve with any relevant flags to filter
					CurveFilter.Add(InCurveName, FilterFlags);
				}

				if(Flags != UE::Anim::ECurveElementFlags::None)
				{
					// Add curve with any relevant flags to bulk flags
					CurveFlags.Add(InCurveName, Flags);
				}
			});

			// Now add any filtered curves we didnt find in metadata
			CurveFilter.AppendNames(FilterCurves);
		}
	}

	if (USkeletalMesh* SkeletalMesh = AssetSkeletalMesh.Get())
	{
		// Override any metadata with the skeletal mesh
		if(const UAnimCurveMetaData* MetaDataUserData = SkeletalMesh->GetAssetUserData<UAnimCurveMetaData>())
		{
			UE::Anim::FBulkCurveFlags MeshCurveFlags;

			// Apply morph target flags to any morph curves
			MetaDataUserData->ForEachCurveMetaData([&MeshCurveFlags](FName InCurveName, const FCurveMetaData& InCurveMetaData)
			{
				UE::Anim::ECurveElementFlags Flags = UE::Anim::ECurveElementFlags::None;
				if(InCurveMetaData.Type.bMaterial)
				{
					Flags |= UE::Anim::ECurveElementFlags::Material;
				}
				if(InCurveMetaData.Type.bMorphtarget)
				{
					Flags |= UE::Anim::ECurveElementFlags::MorphTarget;
				}

				if(Flags != UE::Anim::ECurveElementFlags::None)
				{
					MeshCurveFlags.Add(InCurveName, Flags);
				}
			});

			UE::Anim::FNamedValueArrayUtils::Union(CurveFlags, MeshCurveFlags);
		}
	}

	RegenerateSerialNumber();
}

void FBoneContainer::RegenerateSerialNumber()
{
	// Bump the serial number
	SerialNumber++;

	// Skip zero as this is used to indicate an invalid bone container
	if(SerialNumber == 0)
	{
		SerialNumber++;
	}
}

const FRetargetSourceCachedData& FBoneContainer::GetRetargetSourceCachedData(const FName& InRetargetSourceName) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));
	const TArray<FTransform>& RetargetTransforms = AssetSkeleton.GetEvenIfUnreachable()->GetRefLocalPoses(InRetargetSourceName);
	return GetRetargetSourceCachedData(InRetargetSourceName, FSkeletonRemapping(), RetargetTransforms);
}

const FRetargetSourceCachedData& FBoneContainer::GetRetargetSourceCachedData(
	const FName& InSourceName,
	const FSkeletonRemapping& InRemapping,
	const TArray<FTransform>& InRetargetTransforms) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));
	
	const USkeleton* SourceSkeleton = InRemapping.IsValid() ? InRemapping.GetSourceSkeleton().Get() : AssetSkeleton.Get();
	check(SourceSkeleton);
	
	// Invalid retarget source names (not found on the skeleton) are internally treated as None, so we do the same
	FName RetargetSourceKey = InSourceName;
	if (!SourceSkeleton->AnimRetargetSources.Contains(InSourceName))
	{
		RetargetSourceKey = NAME_None;
	}
	
	// Construct a key from the skeleton and the retarget source since both are necessary for uniqueness
	FRetargetSourceCachedDataKey LUTKey(Cast<UObject>(SourceSkeleton), RetargetSourceKey);
	FRetargetSourceCachedData* RetargetSourceCachedData = RetargetSourceCachedDataLUT.Find(LUTKey);
	if (!RetargetSourceCachedData)
	{
		RetargetSourceCachedData = &RetargetSourceCachedDataLUT.Add(LUTKey);

		// Build Cached Data for OrientAndScale retargeting.

		const TArray<FTransform>& AuthoredOnRefSkeleton = InRetargetTransforms;
		const TArray<FTransform>& PlayingOnRefSkeleton = GetRefPoseArray(); 
		const int32 CompactPoseNumBones = GetCompactPoseNumBones();

		RetargetSourceCachedData->CompactPoseIndexToOrientAndScaleIndex.Reset();

		for (int32 CompactBoneIndex = 0; CompactBoneIndex < CompactPoseNumBones; CompactBoneIndex++)
		{
			const int32 TargetSkeletonBoneIndex = CompactPoseToSkeletonIndex[CompactBoneIndex];
			const int32 SourceSkeletonBoneIndex = InRemapping.IsValid() ? InRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex) : TargetSkeletonBoneIndex;

			// Check what retarget mode to use for the translational retargeting for this specific bone.
			const bool bUseSourceRetargetModes = AssetSkeleton.GetEvenIfUnreachable()->GetUseRetargetModesFromCompatibleSkeleton();
			const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
				bUseSourceRetargetModes,
				SourceSkeletonBoneIndex,
				TargetSkeletonBoneIndex,
				SourceSkeleton,
				AssetSkeleton.GetEvenIfUnreachable(),
				bDisableRetargeting);

			if (RetargetMode == EBoneTranslationRetargetingMode::OrientAndScale)
			{
				if(AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
				{
					const FVector SourceSkelTrans = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex].GetTranslation();
					FVector TargetSkelTrans;
					if (RefPoseOverride.IsValid())
					{
						TargetSkelTrans = RefPoseOverride->RefBonePoses[BoneIndicesArray[CompactBoneIndex]].GetTranslation();
					}
					else
					{
						TargetSkelTrans = PlayingOnRefSkeleton[BoneIndicesArray[CompactBoneIndex]].GetTranslation();
					}

					// If translations are identical, we don't need to do any retargeting
					if (!SourceSkelTrans.Equals(TargetSkelTrans, BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION))
					{
						const float SourceSkelTransLength = SourceSkelTrans.Size();
						const float TargetSkelTransLength = TargetSkelTrans.Size();

						// this only works on non zero vectors.
						if (!FMath::IsNearlyZero(SourceSkelTransLength * TargetSkelTransLength))
						{
							const FVector SourceSkelTransDir = SourceSkelTrans / SourceSkelTransLength;
							const FVector TargetSkelTransDir = TargetSkelTrans / TargetSkelTransLength;

							const FQuat DeltaRotation = FQuat::FindBetweenNormals(SourceSkelTransDir, TargetSkelTransDir);
							const float Scale = TargetSkelTransLength / SourceSkelTransLength;
							const int32 OrientAndScaleIndex = RetargetSourceCachedData->OrientAndScaleData.Add(FOrientAndScaleRetargetingCachedData(DeltaRotation, Scale, SourceSkelTrans, TargetSkelTrans));

							// initialize CompactPoseBoneIndex to OrientAndScale Index LUT on demand
							if (RetargetSourceCachedData->CompactPoseIndexToOrientAndScaleIndex.Num() == 0)
							{
								RetargetSourceCachedData->CompactPoseIndexToOrientAndScaleIndex.Init(INDEX_NONE, CompactPoseNumBones);
							}

							RetargetSourceCachedData->CompactPoseIndexToOrientAndScaleIndex[CompactBoneIndex] = OrientAndScaleIndex;
						}
					}
				}
			}
		}
	}

	return *RetargetSourceCachedData;
}

int32 FBoneContainer::GetPoseBoneIndexForBoneName(const FName& BoneName) const
{
	checkSlow( IsValid() );
	return RefSkeleton->FindBoneIndex(BoneName);
}

int32 FBoneContainer::GetParentBoneIndex(const int32 BoneIndex) const
{
	checkSlow( IsValid() );
	checkSlow(BoneIndex != INDEX_NONE);
	return RefSkeleton->GetParentIndex(BoneIndex);
}

FCompactPoseBoneIndex FBoneContainer::GetParentBoneIndex(const FCompactPoseBoneIndex& BoneIndex) const
{
	checkSlow(IsValid());
	checkSlow(BoneIndex != INDEX_NONE);
	return CompactPoseParentBones[BoneIndex.GetInt()];
}

int32 FBoneContainer::GetDepthBetweenBones(const int32 BoneIndex, const int32 ParentBoneIndex) const
{
	checkSlow( IsValid() );
	checkSlow( BoneIndex != INDEX_NONE );
	return RefSkeleton->GetDepthBetweenBones(BoneIndex, ParentBoneIndex);
}

bool FBoneContainer::BoneIsChildOf(const int32 BoneIndex, const int32 ParentBoneIndex) const
{
	checkSlow( IsValid() );
	checkSlow( (BoneIndex != INDEX_NONE) && (ParentBoneIndex != INDEX_NONE) );
	return RefSkeleton->BoneIsChildOf(BoneIndex, ParentBoneIndex);
}

bool FBoneContainer::BoneIsChildOf(const FCompactPoseBoneIndex& BoneIndex, const FCompactPoseBoneIndex& ParentBoneIndex) const
{
	checkSlow(IsValid());
	checkSlow((BoneIndex != INDEX_NONE) && (ParentBoneIndex != INDEX_NONE));

	// Bones are in strictly increasing order.
	// So child must have an index greater than its parent.
	if (BoneIndex > ParentBoneIndex)
	{
		FCompactPoseBoneIndex SearchBoneIndex = GetParentBoneIndex(BoneIndex);
		do
		{
			if (SearchBoneIndex == ParentBoneIndex)
			{
				return true;
			}
			SearchBoneIndex = GetParentBoneIndex(SearchBoneIndex);

		} while (SearchBoneIndex != INDEX_NONE);
	}

	return false;
}

void FBoneContainer::RemapFromSkelMesh(USkeletalMesh const & SourceSkeletalMesh, USkeleton& TargetSkeleton)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));
	
	const FSkeletonToMeshLinkup& LinkupTable = TargetSkeleton.FindOrAddMeshLinkupData(&SourceSkeletalMesh);

	// Copy LinkupTable arrays for now.
	// @laurent - Long term goal is to trim that down based on LOD, so we can get rid of the BoneIndicesArray and branch cost of testing if PoseBoneIndex is in that required bone index array.
	SkeletonToPoseBoneIndexArray = LinkupTable.SkeletonToMeshTable;
	PoseToSkeletonBoneIndexArray = LinkupTable.MeshToSkeletonTable;
}

void FBoneContainer::RemapFromSkeleton(USkeleton const & SourceSkeleton)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/BoneContainer"));
	// Map SkeletonBoneIndex to the SkeletalMesh Bone Index, taking into account the required bone index array.
	SkeletonToPoseBoneIndexArray.Init(INDEX_NONE, SourceSkeleton.GetRefLocalPoses().Num());
	for(int32 Index=0; Index<BoneIndicesArray.Num(); Index++)
	{
		int32 const & PoseBoneIndex = BoneIndicesArray[Index];
		SkeletonToPoseBoneIndexArray[PoseBoneIndex] = PoseBoneIndex;
	}

	// Skeleton to Skeleton mapping...
	PoseToSkeletonBoneIndexArray = SkeletonToPoseBoneIndexArray;
}

TArray<uint16> const& FBoneContainer::GetUIDToArrayLookupTable() const
{
	static TArray<uint16> Dummy;
	return Dummy;
}

int32 FBoneContainer::GetUIDToArrayIndexLookupTableValidCount() const
{
	return 0;
}

TArray<FAnimCurveType> const& FBoneContainer::GetUIDToCurveTypeLookupTable() const
{
	static TArray<FAnimCurveType> Dummy;
	return Dummy;
}

TArray<SmartName::UID_Type> const& FBoneContainer::GetUIDToArrayLookupTableBackup() const
{
	static TArray<SmartName::UID_Type> Dummy;
	return Dummy;
}
