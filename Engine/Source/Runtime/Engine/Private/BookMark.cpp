// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/BookMark.h"
#include "Engine/BookMark2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BookMark)

UBookMark::UBookMark(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UBookMark2D::UBookMark2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

