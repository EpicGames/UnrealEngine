// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Brush.cpp: Brush Actor implementation
=============================================================================*/

#include "Engine/Brush.h"
#include "Engine/Polys.h"
#include "Engine/Level.h"
#include "EngineLogs.h"
#include "Model.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "Engine/BrushBuilder.h"
#include "Components/BrushComponent.h"
#include "ActorEditorUtils.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Brush)

#if WITH_EDITOR
#include "Editor.h"
#else
#include "Engine/Engine.h"
#endif

#if WITH_EDITOR
/** Define static delegate */
ABrush::FOnBrushRegistered ABrush::OnBrushRegistered;

/** An array to keep track of all the levels that need rebuilding. This is checked via NeedsRebuild() in the editor tick and triggers a csg rebuild. */
TArray< TWeakObjectPtr< ULevel > > ABrush::LevelsToRebuild;

/** Whether BSP regeneration should be suppressed or not */
bool ABrush::bSuppressBSPRegeneration = false;

// Debug purposes only; an attempt to catch the cause of UE-36265
const TCHAR* ABrush::GGeometryRebuildCause = nullptr;

namespace BrushUtils
{
	bool CanDeleteOrReplaceCommon(const ABrush* InActor, FText& OutReason)
	{
		if (FActorEditorUtils::IsABuilderBrush(InActor))
		{
			OutReason = NSLOCTEXT("Brush", "CanDeleteOrReplace_Error_BuilderBrush", "Can't delete or replace a builder brush.");
			return false;
		}

		return true;
	}
}

namespace BrushNavmeshGenerationCVars
{
	static bool bForceNavmeshGenerationOnStaticBrush = true;
	static FAutoConsoleVariableRef CVarForceNavmeshGenerationOnStaticBrush(
		TEXT("brush.ForceNavmeshGenerationOnStaticBrush"), 
		bForceNavmeshGenerationOnStaticBrush, 
		TEXT("Force exporting static brush to level's static navigable geometry through BSP regardless of bCanEverAffectNavigation. Enabled by default to be backward compatible with brushes on legacy maps."), 
		ECVF_Default);
}
#endif

DEFINE_LOG_CATEGORY_STATIC(LogBrush, Log, All);

ABrush::ABrush(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BrushComponent = CreateDefaultSubobject<UBrushComponent>(TEXT("BrushComponent0"));
	BrushComponent->Mobility = EComponentMobility::Static;
	BrushComponent->SetGenerateOverlapEvents(false);
	BrushComponent->SetCanEverAffectNavigation(false);

	RootComponent = BrushComponent;
	
	SetHidden(true);
	bNotForClientOrServer = false;
	SetCanBeDamaged(false);
	bCollideWhenPlacing = true;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding;
}

#if WITH_EDITOR
void ABrush::PostEditMove(bool bFinished)
{
	bInManipulation = !bFinished;

	if( BrushComponent )
	{
		BrushComponent->ReregisterComponent();
	}

	Super::PostEditMove(bFinished);
}

void ABrush::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// Prior to reregistering the BrushComponent (done in the Super), request an update to the Body Setup to take into account any change
	// in the mirroring of the Actor. This will actually be updated when the component is reregistered.
	if (BrushComponent && PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetName() == TEXT("RelativeScale3D"))
	{
		BrushComponent->RequestUpdateBrushCollision();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

FName ABrush::GetCustomIconName() const
{
	if (BrushType == Brush_Add)
	{
		static const FName AdditiveIconName("ClassIcon.BrushAdditive");
		return AdditiveIconName;
	}
	else if (BrushType == Brush_Subtract)
	{
		static const FName SubtactiveIconName("ClassIcon.BrushSubtractive");
		return SubtactiveIconName;
	}

	return NAME_None;
}

void ABrush::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if(Brush)
	{
		Brush->BuildBound();
	}

	bool bIsBuilderBrush = FActorEditorUtils::IsABuilderBrush( this );
	if (!bIsBuilderBrush && (BrushType == Brush_Default))
	{
		// Don't allow non-builder brushes to be set to the default brush type
		BrushType = Brush_Add;
	}
	else if (bIsBuilderBrush && (BrushType != Brush_Default))
	{
		// Don't allow the builder brush to be set to the anything other than the default brush type
		BrushType = Brush_Default;
	}

	if (!bSuppressBSPRegeneration && IsStaticBrush() && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive && GUndo)
	{
		// Don't rebuild BSP if only the actor label has changed
		static const FName ActorLabelName("ActorLabel");
		if (!PropertyChangedEvent.Property || PropertyChangedEvent.Property->GetFName() != ActorLabelName)
		{
			// BSP can only be rebuilt during a transaction
			GEditor->RebuildAlteredBSP();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ABrush::CopyPosRotScaleFrom( ABrush* Other )
{
	check(BrushComponent);
	check(Other);
	check(Other->BrushComponent);

	SetActorLocationAndRotation(Other->GetActorLocation(), Other->GetActorRotation(), false);
	if( GetRootComponent() != NULL )
	{
		SetPivotOffset(Other->GetPivotOffset());
	}

	if(Brush)
	{
		Brush->BuildBound();
	}

	ReregisterAllComponents();
}

bool ABrush::NeedsRebuild(TArray< TWeakObjectPtr< ULevel > >* OutLevels)
{
	LevelsToRebuild.RemoveAllSwap([](const TWeakObjectPtr<ULevel>& Level) { return !Level.IsValid(); });

	if (OutLevels)
	{
		*OutLevels = LevelsToRebuild;
	}

	return(LevelsToRebuild.Num() > 0);
}

void ABrush::SetNeedRebuild(ULevel* InLevel)
{
	if (InLevel)
	{
		LevelsToRebuild.AddUnique(InLevel);
	}
}

bool ABrush::CanEverAffectBSP() const
{
	return !(IsVolumeBrush() || FActorEditorUtils::IsABuilderBrush(this));
}

bool ABrush::ShouldExportStaticNavigableGeometry() const
{
	if (BrushComponent)
	{
		if (!CanEverAffectBSP())
		{
			return false;
		}

		// non-static brush should participate in BSP Rebuild for setting up other data from the existing code.
		// However, they will not export static navigable geometry data to the level.
		// Instead, they set up their Body Setup and Body Instance and register to the navigation system
		// on a per-component basis like other components.
		if (!IsStaticBrush())
		{
			return false;
		}

		// Nav relevancy (i.e., IsNavigationRelevant()) only applies to registration to navigation system, so don't check here
		if (!(BrushNavmeshGenerationCVars::bForceNavmeshGenerationOnStaticBrush || BrushComponent->CanEverAffectNavigation()))
		{
			return false;
		}

		return true;
	}

	return false;
}

void ABrush::InitPosRotScale()
{
	check(BrushComponent);

	SetActorLocationAndRotation(FVector::ZeroVector, FQuat::Identity, false);
	SetPivotOffset( FVector::ZeroVector );
}

void ABrush::SetIsTemporarilyHiddenInEditor( bool bIsHidden )
{
	if (IsTemporarilyHiddenInEditor() != bIsHidden)
	{
		Super::SetIsTemporarilyHiddenInEditor(bIsHidden);
		
		ULevel* Level = GetLevel();
		UModel* Model = Level ? ToRawPtr(Level->Model) : nullptr;

		if (Level && Model)
		{
			bool bAnySurfaceWasFound = false;
			for (int32 SurfIndex = 0; SurfIndex < Model->Surfs.Num(); ++SurfIndex)
			{
				FBspSurf &Surf = Model->Surfs[SurfIndex ];

				if (Surf.Actor == this)
				{
					Model->ModifySurf( SurfIndex , false );
					bAnySurfaceWasFound = true;
					Surf.bHiddenEdTemporary = bIsHidden;
				}
			}

			if (bAnySurfaceWasFound)
			{
				Level->UpdateModelComponents();
				Level->InvalidateModelSurface();
			}
		}
	}
}

bool ABrush::SetIsHiddenEdLayer(bool bIsHiddenEdLayer)
{
	if (Super::SetIsHiddenEdLayer(bIsHiddenEdLayer))
	{
		ULevel* Level = GetLevel();
		UModel* Model = Level ? ToRawPtr(Level->Model) : nullptr;
		if (Level && Model)
		{
			bool bAnySurfaceWasFound = false;
			for (FBspSurf& Surf : Model->Surfs)
			{
				if (Surf.Actor == this)
				{
					Surf.bHiddenEdLayer = bIsHiddenEdLayer;
					bAnySurfaceWasFound = true;
				}
			}

			if (bAnySurfaceWasFound)
			{
				Level->UpdateModelComponents();
				Model->InvalidSurfaces = true;
			}
		}
		return true;
	}
	return false;
}

bool ABrush::SupportsLayers() const
{
	return !FActorEditorUtils::IsABuilderBrush(this) && Super::SupportsLayers();
}

bool ABrush::SupportsExternalPackaging() const
{
	// Base class ABrush actors do not support OFPA
	return GetClass() != ABrush::StaticClass() && Super::SupportsExternalPackaging();
}

bool ABrush::CanDeleteSelectedActor(FText& OutReason) const
{
	if (!Super::CanDeleteSelectedActor(OutReason))
	{
		return false;
	}

	return BrushUtils::CanDeleteOrReplaceCommon(this, OutReason);
}

bool ABrush::CanReplaceSelectedActor(FText& OutReason) const
{
	if (!Super::CanReplaceSelectedActor(OutReason))
	{
		return false;
	}

	return BrushUtils::CanDeleteOrReplaceCommon(this, OutReason);
}

bool ABrush::IsActorLabelEditable() const
{
	if (!Super::IsActorLabelEditable())
	{
		return false;
	}

	return !FActorEditorUtils::IsABuilderBrush(this);
}

void ABrush::PostLoad()
{
	Super::PostLoad();

	if (BrushBuilder && BrushBuilder->GetOuter() != this)
	{
		BrushBuilder = DuplicateObject<UBrushBuilder>(BrushBuilder, this);
	}

	// Assign the default material to brush polys with NULL material references.
	if ( Brush && Brush->Polys )
	{
		if ( IsStaticBrush() )
		{
			for( int32 PolyIndex = 0 ; PolyIndex < Brush->Polys->Element.Num() ; ++PolyIndex )
			{
				FPoly& CurrentPoly = Brush->Polys->Element[PolyIndex];
				if ( !CurrentPoly.Material )
				{
					CurrentPoly.Material = UMaterial::GetDefaultMaterial(MD_Surface);
				}
			}
		}

#if WITH_EDITOR
		// Fix up corrupted brush references.
		if (BrushComponent)
		{
			if (!BrushComponent->Brush)
			{
				UE_LOGF(LogBrush, Warning, "Component inside %ls had missing brush, but actor had it. "
					"This should not happen. Resave package '%ls' to remove this warning.",
					*GetNameSafe(this), *GetNameSafe(GetPackage()));
				BrushComponent->Brush = Brush;
			}
		}

		// Fix up any broken poly normals.
		// They have not been getting fixed up after vertex editing since at least UE2!
		for(FPoly& Poly : Brush->Polys->Element)
		{
			FVector3f Normal = Poly.Normal;
			if(!Poly.CalcNormal())
			{
				if(!Poly.Normal.Equals(Normal))
				{
					UE_LOGF(LogBrush, Log, "%ls had invalid poly normals which have been fixed. Resave the level '%ls' to remove this warning.", *Brush->GetName(), *GetLevel()->GetOuter()->GetName());
					if(IsStaticBrush())
					{
						UE_LOGF(LogBrush, Log, "%ls had invalid poly normals which have been fixed. Resave the level '%ls' to remove this warning.", *Brush->GetName(), *GetLevel()->GetOuter()->GetName());

						// Flag BSP as needing rebuild
						SetNeedRebuild(GetLevel());
					}
				}
			}
		}
#endif

		// if the polys of the brush have the wrong outer, fix it up to be the UModel (my Brush member)
		// UModelFactory::FactoryCreateText was passing in the ABrush as the Outer instead of the UModel
		if (Brush->Polys->GetOuter() == this)
		{
			// Complete loading of the polys before renaming
			Brush->Polys->ConditionalPostLoad();
			Brush->Polys->Rename(*Brush->Polys->GetName(), Brush);
		}
	}

	if ( BrushComponent && !BrushComponent->BrushBodySetup )
	{
		UE_LOGF(LogPhysics, Log, "%ls does not have BrushBodySetup. No collision.", *GetName());
	}
}

void ABrush::Destroyed()
{
	Super::Destroyed();

	if(GIsEditor && IsStaticBrush() && !GetWorld()->IsGameWorld())
	{
		// Trigger a csg rebuild if we're in the editor.
		SetNeedRebuild(GetLevel());
	}
}

void ABrush::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if ( GIsEditor )
	{
		OnBrushRegistered.Broadcast(this);
	}
}
#endif

bool ABrush::IsLevelBoundsRelevant() const
{
	// exclude default brush
	ULevel* Level = GetLevel();
	return (Level && this != Level->Actors[1].Get());
}

bool ABrush::IsComponentRelevantForNavigation(UActorComponent* Component) const
{
	if (BrushComponent && BrushComponent == Component)
	{
		// When Brush changes from movable to static, it doesn't reset its Body Setup and Body Instance,
		// which causes its geometric data be exported twice, both BSP Level and Component Registration
		// So we need to mark it as nav irrelevant
		if (IsStaticBrush())
		{
			return false;
		}
	}

	return Super::IsComponentRelevantForNavigation(Component);
}

void ABrush::RebuildNavigationData()
{
	// empty in base class
}

FColor ABrush::GetWireColor() const
{
	FColor Color = GEngine->C_BrushWire;

	if( IsStaticBrush() )
	{
		Color = bColored ?						BrushColor :
				BrushType == Brush_Subtract ?	GEngine->C_SubtractWire :
				BrushType != Brush_Add ?		GEngine->C_BrushWire :
				(PolyFlags & PF_Portal) ?		GEngine->C_SemiSolidWire :
				(PolyFlags & PF_NotSolid) ?		GEngine->C_NonSolidWire :
				(PolyFlags & PF_Semisolid) ?	GEngine->C_ScaleBoxHi :
												GEngine->C_AddWire;
	}
	else if( IsVolumeBrush() )
	{
		Color = bColored ? BrushColor : GEngine->C_Volume;
	}
	else if( IsBrushShape() )
	{
		Color = bColored ? BrushColor : GEngine->C_BrushShape;
	}

	return Color;
}

bool ABrush::IsStaticBrush() const
{
	return BrushComponent && (BrushComponent->Mobility == EComponentMobility::Static);
}

#if WITH_EDITOR
bool ABrush::Modify(bool bAlwaysMarkDirty)
{
	bool bSavedToTransactionBuffer = Super::Modify(bAlwaysMarkDirty);

	if(Brush)
	{
		bSavedToTransactionBuffer = Brush->Modify(bAlwaysMarkDirty) || bSavedToTransactionBuffer;
	}
	return bSavedToTransactionBuffer;
}
#endif
