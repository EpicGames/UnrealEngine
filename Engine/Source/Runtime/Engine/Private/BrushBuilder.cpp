// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UBrushBuilder.cpp: UnrealEd brush builder.
=============================================================================*/

#include "Engine/BrushBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrushBuilder)

/*-----------------------------------------------------------------------------
	UBrushBuilder.
-----------------------------------------------------------------------------*/
UBrushBuilder::UBrushBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BitmapFilename = TEXT("BBGeneric");
	ToolTip = TEXT("BrushBuilderName_Generic");
	NotifyBadParams = true;
}

