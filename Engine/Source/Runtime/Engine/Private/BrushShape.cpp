// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/BrushShape.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/CollisionProfile.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrushShape)

ABrushShape::ABrushShape(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	GetBrushComponent()->AlwaysLoadOnClient = true;
	GetBrushComponent()->AlwaysLoadOnServer = false;

}


