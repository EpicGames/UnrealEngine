// Copyright Epic Games, Inc. All Rights Reserved.

#include "BufferVisualizationData.h"

#include "Materials/MaterialInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "ShaderCore.h"

#define LOCTEXT_NAMESPACE "FBufferVisualizationData"

static FBufferVisualizationData GBufferVisualizationData;

static TAutoConsoleVariable<int32> BufferVisualizationDumpFramesAsHDR(
	TEXT("r.BufferVisualizationDumpFramesAsHDR"),
	0,
	TEXT("When saving out buffer visualization materials in a HDR capable format\n")
	TEXT("0: Do not override default save format.\n")
	TEXT("1: Force HDR format for buffer visualization materials."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarBufferVisualizationDumpFrames(
	TEXT("r.BufferVisualizationDumpFrames"),
	0,
	TEXT("When screenshots or movies dumps are requested, also save out dumps of the current buffer visualization materials\n")
	TEXT("0:off (default)\n")
	TEXT("1:on"),
	ECVF_RenderThreadSafe);

void FBufferVisualizationData::Initialize()
{
	if (!bIsInitialized)
	{
		if (AllowDebugViewmodes())
		{
			check(MaterialMap.Num() == 0);
			check(MaterialMapFromMaterialName.Num() == 0);

			const FConfigSection* MaterialSection = GConfig->GetSection( TEXT("Engine.BufferVisualizationMaterials"), false, GEngineIni );

			if (MaterialSection != NULL)
			{
				for (FConfigSection::TConstIterator It(*MaterialSection); It; ++It)
				{
					FString EnabledCVar;
					if (FParse::Value(*It.Value().GetValue(), TEXT("Display="), EnabledCVar, true))
					{
						IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*EnabledCVar);
						if (CVar && !CVar->GetBool())
						{
							continue;
						}
					}

					FString MaterialName;
					if( FParse::Value( *It.Value().GetValue(), TEXT("Material="), MaterialName, true ) )
					{
						UMaterialInterface* Material = LoadObject<UMaterialInterface>(NULL, *MaterialName);
			
						if (Material)
						{
							FText DisplayName;
							FParse::Value(*It.Value().GetValue(), TEXT("Name="), DisplayName, TEXT("Engine.BufferVisualizationMaterials"));

							bool bApplyAutoExposure = false;
							FParse::Bool(*It.Value().GetValue(), TEXT("ApplyAutoExposure="), bApplyAutoExposure);

							bool bEnableOnMobile = true;
							FParse::Bool(*It.Value().GetValue(), TEXT("EnableOnMobile="), bEnableOnMobile);

							Material->AddToRoot();
							Record& Rec = MaterialMap.Add(It.Key(), Record());
							Rec.Name = It.Key().GetPlainNameString();
							Rec.Material = Material;
							Rec.bApplyAutoExposure = bApplyAutoExposure;
							Rec.DisplayName = DisplayName;
							Rec.bEnableOnMobile = bEnableOnMobile;

							MaterialMapFromMaterialName.Add(Material->GetFName(), Rec);
						}
					}
				}
			}

			ConfigureConsoleCommand();
		}

		bIsInitialized = true;
	}
}

void FBufferVisualizationData::ConfigureConsoleCommand()
{
	struct Local
	{
		FString& Message;

		Local(FString& OutString)
			: Message(OutString)
		{
		}

		void ProcessValue(const FString& InMaterialName, const UMaterialInterface* InMaterial, const FText& InDisplayName)
		{
			Message += FString("\n  ");
			Message += InMaterialName;
		}
	};

	FString AvailableVisualizationMaterials;
	Local It(AvailableVisualizationMaterials);
	IterateOverAvailableMaterials(It);

	ConsoleDocumentationVisualizationMode = TEXT("When the viewport view-mode is set to 'Buffer Visualization', this command specifies which of the various channels to display. Values entered other than the allowed values shown below will be ignored.");
	ConsoleDocumentationVisualizationMode += AvailableVisualizationMaterials;

	IConsoleManager::Get().RegisterConsoleVariable(
		GetVisualizationTargetConsoleCommandName(),
		TEXT(""),
		*ConsoleDocumentationVisualizationMode,
		ECVF_Cheat);

	ConsoleDocumentationOverviewTargets = TEXT("Specify the list of post process materials that can be used in the buffer visualization overview. Put nothing between the commas to leave a gap.\n\n\tChoose from:\n");
	ConsoleDocumentationOverviewTargets += AvailableVisualizationMaterials;

	IConsoleManager::Get().RegisterConsoleVariable(
		TEXT("r.BufferVisualizationOverviewTargets"),
		TEXT("BaseColor,Specular,SubsurfaceColor,WorldNormal,SeparateTranslucencyRGB,,,WorldTangent,SeparateTranslucencyA,,,Opacity,SceneDepth,Roughness,Metallic,ShadingModel,,SceneDepthWorldUnits,SceneColor,PreTonemapHDRColor,PostTonemapHDRColor"),
		*ConsoleDocumentationOverviewTargets,
		ECVF_Default
		);
}

const FBufferVisualizationData::Record* FBufferVisualizationData::GetRecord(FName InMaterialName) const
{
	// Get UMaterial from the TMaterialMap FName
	if (const Record* Result = MaterialMap.Find(InMaterialName))
	{
		return Result;
	}
	// Get UMaterial from the UMaterial FName
	// Almost all BufferVisualizationData variables contain a FMaterial with its same FName.
	// But not all, e.g., ShadingModel uses the FMaterial LightingModel. This could confuse the developer depending on which one they are using
	// This "else if" case handles the case in which we look for "LightingModel" rather than "ShadingModel", returning the same value
	else if (const Record* ResultFromMaterialName = MaterialMapFromMaterialName.Find(InMaterialName))
	{
		return ResultFromMaterialName;
	}
	// Not found
	else
	{
		return nullptr;
	}
}

UMaterialInterface* FBufferVisualizationData::GetMaterial(FName InMaterialName) const
{
	if (const Record* Result = GetRecord(InMaterialName))
	{
		return Result->Material;
	}
	else
	{
		return nullptr;
	}
}

bool FBufferVisualizationData::GetMaterialApplyAutoExposure(FName InMaterialName) const
{
	if (const Record* Result = GetRecord(InMaterialName))
	{
		return Result->bApplyAutoExposure;
	}
	else
	{
		return false;
	}
}

bool FBufferVisualizationData::GetMaterialEnableOnMobile(FName InMaterialName) const
{
	if (const Record* Result = GetRecord(InMaterialName))
	{
		return Result->bEnableOnMobile;
	}
	else
	{
		return false;
	}
}

FText FBufferVisualizationData::GetMaterialDisplayName(FName InMaterialName) const
{
	if (const Record* Result = GetRecord(InMaterialName))
	{
		return Result->DisplayName;
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText FBufferVisualizationData::GetMaterialDefaultDisplayName()
{
	return LOCTEXT("BufferVisualization", "Overview");
}

void FBufferVisualizationData::SetCurrentOverviewMaterialNames(const FString& InNameList)
{
	CurrentOverviewMaterialNames = InNameList;
}

bool FBufferVisualizationData::IsDifferentToCurrentOverviewMaterialNames(const FString& InNameList)
{
	return InNameList != CurrentOverviewMaterialNames;
}

TArray<UMaterialInterface*>& FBufferVisualizationData::GetOverviewMaterials()
{
	return OverviewMaterials;
}

FBufferVisualizationData& GetBufferVisualizationData()
{
	if (!GBufferVisualizationData.IsInitialized())
	{
		GBufferVisualizationData.Initialize();
	}

	return GBufferVisualizationData;
}

#undef LOCTEXT_NAMESPACE
