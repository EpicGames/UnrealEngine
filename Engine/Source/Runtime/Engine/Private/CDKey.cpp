// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CDKey.cpp: CD Key validation
=============================================================================*/

#include "CDKey.h"
#include "Misc/SecureHash.h"

/*-----------------------------------------------------------------------------
	Internal MD5 Stuff
-----------------------------------------------------------------------------*/

//!! TEMP - need to load cdkey from disk.
#define CDKEY TEXT("54321")

/*-----------------------------------------------------------------------------
	Global CD Key functions
-----------------------------------------------------------------------------*/

FString GetCDKeyHash()
{
	return FMD5::HashAnsiString(CDKEY);
}

FString GetCDKeyResponse( const TCHAR* Challenge )
{
	FString CDKey;
	
	// Get real CD Key
	CDKey = CDKEY;
    
	// Append challenge
	CDKey += Challenge;

	// MD5
	return FMD5::HashAnsiString( *CDKey );
}

