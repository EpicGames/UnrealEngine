// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraActor.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "Kismet/GameplayStatics.h"
#include "Camera/CameraComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraActor)

#define LOCTEXT_NAMESPACE "CameraActor"

//////////////////////////////////////////////////////////////////////////
// ACameraActor

ACameraActor::ACameraActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"));

	// Make the scene component the root component
	RootComponent = SceneComponent;
	
	// Setup camera defaults
	CameraComponent = CreateDefaultSubobject<UCameraComponent>(TEXT("CameraComponent"));
	CameraComponent->FieldOfView = 90.0f;
	CameraComponent->bConstrainAspectRatio = true;
	CameraComponent->AspectRatio = 1.777778f;
	CameraComponent->PostProcessBlendWeight = 1.0f;
	
	CameraComponent->SetupAttachment(SceneComponent);

	// Initialize deprecated properties (needed for backwards compatibility due to delta serialization)
	FOVAngle_DEPRECATED = 90.0f;
	bConstrainAspectRatio_DEPRECATED = true;
	AspectRatio_DEPRECATED = 1.777778f;
	PostProcessBlendWeight_DEPRECATED = 1.0f;
	// End of deprecated property initialization
}

void ACameraActor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if ((Ar.UEVer() < VER_UE4_CAMERA_ACTOR_USING_CAMERA_COMPONENT) && Ar.IsLoading())
	{
		CameraComponent->bConstrainAspectRatio = bConstrainAspectRatio_DEPRECATED;
		CameraComponent->ProjectionMode = ECameraProjectionMode::Perspective;
		CameraComponent->AspectRatio = AspectRatio_DEPRECATED;
		CameraComponent->FieldOfView = FOVAngle_DEPRECATED;
		CameraComponent->PostProcessBlendWeight = PostProcessBlendWeight_DEPRECATED;
		CameraComponent->PostProcessSettings = PostProcessSettings_DEPRECATED;
	}
}

#if WITH_EDITOR
void ACameraActor::PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph)
{
	USceneComponent* OldRoot = RootComponent;
	USceneComponent* OldAttachParent = OldRoot ? OldRoot->GetAttachParent() : nullptr;
	const FName OldSocketName = OldRoot ? OldRoot->GetAttachSocketName() : FName();

	Super::PostLoadSubobjects(OuterInstanceGraph);
	
	if (GetLinkerUEVersion() < VER_UE4_CAMERA_ACTOR_USING_CAMERA_COMPONENT)
	{
		CameraComponent->SetupAttachment(OldAttachParent, OldSocketName);
		OldRoot->SetupAttachment(nullptr);
	}

	if (GetLinkerUEVersion() < VER_UE4_CAMERA_COMPONENT_ATTACH_TO_ROOT)
	{
		RootComponent = SceneComponent;
		if (OldAttachParent != SceneComponent)
		{
			CameraComponent->SetupAttachment(RootComponent);
			RootComponent->SetupAttachment(OldAttachParent, OldSocketName);
		}
	}
}

#endif

USceneComponent* ACameraActor::GetDefaultAttachComponent() const
{
	return CameraComponent;
}

int32 ACameraActor::GetAutoActivatePlayerIndex() const
{
	if (AutoActivateForPlayer != EAutoReceiveInput::Disabled)
	{
		const int32 PlayerIndex = int32(AutoActivateForPlayer.GetValue()) - 1;
		return PlayerIndex;
	}
	else
	{
		return INDEX_NONE;
	}
}

void ACameraActor::BeginPlay()
{
	if (AutoActivateForPlayer != EAutoReceiveInput::Disabled && GetNetMode() != NM_Client)
	{
		const int32 PlayerIndex = GetAutoActivatePlayerIndex();
		
		// Always put it in the pool of available auto-activate cameras.
		GetWorld()->RegisterAutoActivateCamera(this, PlayerIndex);

		// If we find a matching PC, bind to it immediately.
		APlayerController* PC = UGameplayStatics::GetPlayerController(this, PlayerIndex);
		if (PC)
		{
			PC->SetViewTarget(this);
		}
	}

	Super::BeginPlay();
}

#undef LOCTEXT_NAMESPACE


