// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraAnimationHelper.h"
#include "Camera/CameraTypes.h"
#include "Math/RotationMatrix.h"
#include "Math/Vector4.h"

void FCameraAnimationHelper::ApplyOffset(const FMinimalViewInfo& InPOV, const FCameraAnimationHelperOffset& InOffset, FVector& OutLocation, FRotator& OutRotation)
{
	const FRotationMatrix CameraRot(InPOV.Rotation);
	const FRotationMatrix OffsetRot(InOffset.Rotation);

	// Apply translation offset in the camera's local space.
	OutLocation = InPOV.Location + CameraRot.TransformVector(InOffset.Location);

	// Apply rotation offset to camera's local orientation.
	OutRotation = (OffsetRot * CameraRot).Rotator();
}

void FCameraAnimationHelper::ApplyOffset(const FMatrix& UserPlaySpaceMatrix, const FMinimalViewInfo& InPOV, const FCameraAnimationHelperOffset& InOffset, FVector& OutLocation, FRotator& OutRotation)
{
	const FRotationMatrix CameraRot(InPOV.Rotation);
	const FRotationMatrix OffsetRot(InOffset.Rotation);

	// Apply translation offset using the desired space.
	// (it's FMatrix::Identity if the space is World, and whatever value was passed to StartShake if UserDefined)
	OutLocation = InPOV.Location + UserPlaySpaceMatrix.TransformVector(InOffset.Location);

	// Apply rotation offset using the desired space.
	//
	// Compute the transform from camera to play space.
	FMatrix const CameraToPlaySpace = CameraRot * UserPlaySpaceMatrix.Inverse();

	// Compute the transform from shake (applied in playspace) back to camera.
	FMatrix const ShakeToCamera = OffsetRot * CameraToPlaySpace.Inverse();

	// RCS = rotated camera space, meaning camera space after it's been animated.
	// This is what we're looking for, the diff between rotated cam space and regular cam space.
	// Apply the transform back to camera space from the post-animated transform to get the RCS.
	FMatrix const RCSToCamera = CameraToPlaySpace * ShakeToCamera;

	// Now apply to real camera
	OutRotation = (RCSToCamera * CameraRot).Rotator();

	// Math breakdown:
	//
	// ResultRot = RCSToCamera * CameraRot
	// ResultRot = CameraToPlaySpace * ShakeToCamera * CameraRot
	// ResultRot = (CameraToPlaySpace) * OffsetRot * (CameraToPlaySpace^-1) * CameraRot
	//
	// ...where CameraToPlaySpace = (CameraRot * (UserPlaySpaceMatrix^-1))
}

