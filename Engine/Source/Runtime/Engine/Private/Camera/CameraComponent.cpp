// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraComponent.h"

#include "Camera/CameraTypes.h"
#include "CineCameraActor.h"
#include "SceneView.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "Components/StaticMeshComponent.h"
#include "Camera/CameraActor.h"
#include "Engine/Engine.h"
#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Rendering/MotionVectorSimulation.h"
#include "Misc/MapErrors.h"
#include "Components/DrawFrustumComponent.h"
#include "IXRTrackingSystem.h"
#include "IXRCamera.h"
#include "Math/UnitConversion.h"
#include "StaticMeshSceneProxy.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraComponent)

#define LOCTEXT_NAMESPACE "CameraComponent"

FPrimitiveSceneProxy* UCameraProxyMeshComponent::CreateSceneProxy()
{
	class FCameraProxyMeshProxy : public FStaticMeshSceneProxy
	{
	public:
		FCameraProxyMeshProxy(UCameraProxyMeshComponent* Component)
			: FStaticMeshSceneProxy(Component, false)
		{
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			if (View->Family->EngineShowFlags.Cameras)
			{
				return FStaticMeshSceneProxy::GetViewRelevance(View);
			}

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = false;

			return Result;
		}
	};

	return new FCameraProxyMeshProxy(this);
}

//////////////////////////////////////////////////////////////////////////
// UCameraComponent

UCameraComponent::UCameraComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> EditorCameraMesh(TEXT("/Engine/EditorMeshes/MatineeCam_SM"));
		CameraMesh = EditorCameraMesh.Object;
	}

	bUseControllerViewRotation_DEPRECATED = true; // the previous default value before bUsePawnControlRotation replaced this var.
	bCameraMeshHiddenInGame = true;
#endif

	FieldOfView = 90.0f;
	FirstPersonFieldOfView = 90.0f;
	FirstPersonScale = 1.0f;
	AspectRatio = 1.777778f;
	OrthoWidth = DEFAULT_ORTHOWIDTH;
	bAutoCalculateOrthoPlanes = true;
	AutoPlaneShift = 0.0f;
	bUpdateOrthoPlanes = true;
	bUseCameraHeightAsViewTarget = true;
	OrthoNearClipPlane = DEFAULT_ORTHONEARPLANE;
	OrthoFarClipPlane = DEFAULT_ORTHOFARPLANE;
	bConstrainAspectRatio = false;
	bOverrideAspectRatioAxisConstraint = false;
	Overscan = 0.0f;
	bUseFieldOfViewForLOD = true;
	PostProcessBlendWeight = 1.0f;
	bUsePawnControlRotation = false;
	bAutoActivate = true;
	bLockToHmd = true;
	bEnableFirstPersonFieldOfView = false;
	bEnableFirstPersonScale = false;

#if WITH_EDITORONLY_DATA
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;
#endif
}

void UCameraComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
#if WITH_EDITORONLY_DATA
	UpdateProxyMeshTransform();
#endif

	Super::OnUpdateTransform(UpdateTransformFlags, Teleport);
}

#if WITH_EDITORONLY_DATA
void UCameraComponent::PostEditChangeProperty(FPropertyChangedEvent & PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : FName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCameraComponent, bCameraMeshHiddenInGame))
	{
		OnCameraMeshHiddenChanged();
	}

	RefreshVisualRepresentation();
}

void UCameraComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UCameraComponent* This = CastChecked<UCameraComponent>(InThis);
	Collector.AddReferencedObject(This->ProxyMeshComponent);
	Collector.AddReferencedObject(This->DrawFrustum);

	Super::AddReferencedObjects(InThis, Collector);
}

void UCameraComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

	if (ProxyMeshComponent)
	{
		ProxyMeshComponent->DestroyComponent();
	}
	if (DrawFrustum)
	{
		DrawFrustum->DestroyComponent();
	}
}
#endif	// WITH_EDITORONLY_DATA

#if WITH_EDITOR
FText UCameraComponent::GetFilmbackText() const
{
	return FText::FromString(LexToString(FNumericUnit<float>(FieldOfView, EUnit::Degrees)));
}
#endif

void UCameraComponent::OnRegister()
{
#if WITH_EDITORONLY_DATA
	AActor* MyOwner = GetOwner();
	if ((MyOwner != nullptr) && !IsRunningCommandlet())
	{
		if (ProxyMeshComponent == nullptr)
		{
			ProxyMeshComponent = NewObject<UCameraProxyMeshComponent>(MyOwner, NAME_None, RF_Transactional | RF_TextExportTransient);
			ProxyMeshComponent->SetupAttachment(this);
			ProxyMeshComponent->SetIsVisualizationComponent(true);
			ProxyMeshComponent->SetCanEverAffectNavigation(false);
			ProxyMeshComponent->SetStaticMesh(CameraMesh);
			ProxyMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			ProxyMeshComponent->bHiddenInGame = bCameraMeshHiddenInGame;
			ProxyMeshComponent->CastShadow = false;
			ProxyMeshComponent->CreationMethod = CreationMethod;
			ProxyMeshComponent->RegisterComponentWithWorld(GetWorld());
		}

		if (DrawFrustum == nullptr)
		{
			DrawFrustum = NewObject<UDrawFrustumComponent>(MyOwner, NAME_None, RF_Transactional | RF_TextExportTransient);
			DrawFrustum->SetupAttachment(this);
			DrawFrustum->SetIsVisualizationComponent(true);
			DrawFrustum->CreationMethod = CreationMethod;
			DrawFrustum->bFrustumEnabled = bDrawFrustumAllowed;
			DrawFrustum->RegisterComponentWithWorld(GetWorld());
		}
	}

	RefreshVisualRepresentation();
#endif	// WITH_EDITORONLY_DATA

	Super::OnRegister();
}

#if WITH_EDITOR
void UCameraComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	UpdateDrawFrustum();

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}
#endif

#if WITH_EDITORONLY_DATA

void UCameraComponent::PostLoad()
{
	Super::PostLoad();

	const FPackageFileVersion LinkerUEVer = GetLinkerUEVersion();

	if (LinkerUEVer < VER_UE4_RENAME_CAMERA_COMPONENT_VIEW_ROTATION)
	{
		bUsePawnControlRotation = bUseControllerViewRotation_DEPRECATED;
	}
}

 void UCameraComponent::SetCameraMesh(UStaticMesh* Mesh)
 {
	 if (Mesh != CameraMesh)
	 {
		 CameraMesh = Mesh;

		 if (ProxyMeshComponent)
		 {
			 ProxyMeshComponent->SetStaticMesh(Mesh);
		 }
	 }
 }

void UCameraComponent::ResetProxyMeshTransform()
{
	if (ProxyMeshComponent != nullptr)
	{
		ProxyMeshComponent->ResetRelativeTransform();
	}
}

void UCameraComponent::UpdateDrawFrustum()
{
	if (DrawFrustum != nullptr)
	{
		bool bAnythingChanged = false;
		const float FrustumDrawDistance = 1000.0f;
		if (ProjectionMode == ECameraProjectionMode::Perspective)
		{
			if (DrawFrustum->FrustumAngle != FieldOfView ||
				DrawFrustum->FrustumStartDist != 10.f ||
				DrawFrustum->FrustumEndDist != DrawFrustum->FrustumStartDist + FrustumDrawDistance)
			{
				DrawFrustum->FrustumAngle = FieldOfView;
				DrawFrustum->FrustumStartDist = 10.f;
				DrawFrustum->FrustumEndDist = DrawFrustum->FrustumStartDist + FrustumDrawDistance;
				bAnythingChanged = true;
			}
		}
		else
		{
			if (DrawFrustum->FrustumAngle != -OrthoWidth ||
				DrawFrustum->FrustumStartDist != OrthoNearClipPlane ||
				DrawFrustum->FrustumEndDist != FMath::Min(OrthoFarClipPlane - OrthoNearClipPlane, FrustumDrawDistance))
			{
				DrawFrustum->FrustumAngle = -OrthoWidth;
				DrawFrustum->FrustumStartDist = OrthoNearClipPlane;
				DrawFrustum->FrustumEndDist = FMath::Min(OrthoFarClipPlane - OrthoNearClipPlane, FrustumDrawDistance);
				bAnythingChanged = true;
			}
		}

		if (DrawFrustum->FrustumAspectRatio != AspectRatio)
		{
			DrawFrustum->FrustumAspectRatio = AspectRatio;
			bAnythingChanged = true;
		}	
		
		if (bAnythingChanged)
		{
			DrawFrustum->MarkRenderStateDirty();
		}
	}
}

void UCameraComponent::RefreshVisualRepresentation()
{
	UpdateDrawFrustum();

	// Update the proxy camera mesh if necessary
	if (ProxyMeshComponent && ProxyMeshComponent->GetStaticMesh() != CameraMesh)
	{
		ProxyMeshComponent->SetStaticMesh(CameraMesh);
	}

	ResetProxyMeshTransform();
}

void UCameraComponent::UpdateProxyMeshTransform()
{
	if (ProxyMeshComponent)
	{
		FTransform OffsetCamToWorld = AdditiveOffset * GetComponentToWorld();

		ResetProxyMeshTransform();

		FTransform LocalTransform = ProxyMeshComponent->GetRelativeTransform();
		FTransform WorldTransform = LocalTransform * OffsetCamToWorld;

		ProxyMeshComponent->SetWorldTransform(WorldTransform);
	}
}

void UCameraComponent::OverrideFrustumColor(FColor OverrideColor)
{
	if (DrawFrustum != nullptr)
	{
		DrawFrustum->FrustumColor = OverrideColor;
	}
}

void UCameraComponent::RestoreFrustumColor()
{
	if (DrawFrustum != nullptr)
	{
		//@TODO: 
		const FColor DefaultFrustumColor(255, 0, 255, 255);
		DrawFrustum->FrustumColor = DefaultFrustumColor;
		//ACameraActor* DefCam = Cam->GetClass()->GetDefaultObject<ACameraActor>();
		//Cam->DrawFrustum->FrustumColor = DefCam->DrawFrustum->FrustumColor;
	}
}

void UCameraComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::OrthographicCameraDefaultSettings)
	{
		OrthoWidth = 512.0f;
		OrthoNearClipPlane = 0.0f;
		OrthoFarClipPlane = UE_OLD_WORLD_MAX;
	}

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::OrthographicAutoNearFarPlane)
	{
		bAutoCalculateOrthoPlanes = false;
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		PostProcessSettings.OnAfterLoad();
	}
}
#endif	// WITH_EDITORONLY_DATA

void UCameraComponent::OnCameraMeshHiddenChanged()
{
#if WITH_EDITORONLY_DATA
	if (ProxyMeshComponent)
	{
		ProxyMeshComponent->bHiddenInGame = bCameraMeshHiddenInGame;
	}
#endif
}

bool UCameraComponent::IsXRHeadTrackedCamera() const
{
	if (GEngine && GEngine->XRSystem.IsValid() && GetWorld() && GetWorld()->WorldType != EWorldType::Editor)
	{
		IXRTrackingSystem* XRSystem = GEngine->XRSystem.Get();
		auto XRCamera = XRSystem->GetXRCamera();

		if (XRCamera.IsValid())
		{
			if (XRSystem->IsHeadTrackingAllowedForWorld(*GetWorld()))
			{
				return true;
			}
		}
	}

	return false;
}

void UCameraComponent::HandleXRCamera(float DeltaTime)
{
	IXRTrackingSystem* XRSystem = GEngine->XRSystem.Get();
	auto XRCamera = XRSystem->GetXRCamera();

	if (!XRCamera.IsValid())
	{
		return;
	}

	const FTransform ParentWorld = CalcNewComponentToWorld(FTransform());

	XRCamera->SetupLateUpdate(ParentWorld, this, bLockToHmd == 0);

	if (bLockToHmd)
	{
		FQuat Orientation;
		FVector Position;
		if (XRCamera->UpdatePlayerCamera(Orientation, Position, DeltaTime))
		{
			SetRelativeTransform(FTransform(Orientation, Position));
		}
		else
		{
			ResetRelativeTransform();
		}
	}

	XRCamera->OverrideFOV(this->FieldOfView);
}

void UCameraComponent::GetCameraView(float DeltaTime, FMinimalViewInfo& DesiredView)
{
	if (IsXRHeadTrackedCamera())
	{
		HandleXRCamera(DeltaTime);
	}

	if (bUsePawnControlRotation)
	{
		const APawn* OwningPawn = Cast<APawn>(GetOwner());
		const AController* OwningController = OwningPawn ? OwningPawn->GetController() : nullptr;
		if (OwningController && OwningController->IsLocalPlayerController())
		{
			const FRotator PawnViewRotation = OwningPawn->GetViewRotation();
			if (!PawnViewRotation.Equals(GetComponentRotation()))
			{
				SetWorldRotation(PawnViewRotation);
			}
		}
	}
	else if (TObjectPtr<ACineCameraActor> Cam = Cast<ACineCameraActor>(GetOwner()))
	{
		// TODO maybe we should do the lookat logic here instead?  Maybe with some blend amount if that can be determined.
		// Or defer the Camera tick group until the latest it can be.
		if (Cam->LookatTrackingSettings.LastLookatTrackingRotationFrame == GFrameNumber)
		{
			SetWorldRotation(Cam->LookatTrackingSettings.LastLookatTrackingRotation);
		}
	}

	if (bUseAdditiveOffset)
	{
		FTransform OffsetCamToBaseCam = AdditiveOffset;
		FTransform BaseCamToWorld = GetComponentToWorld();
		FTransform OffsetCamToWorld = OffsetCamToBaseCam * BaseCamToWorld;

		DesiredView.Location = OffsetCamToWorld.GetLocation();
		DesiredView.Rotation = OffsetCamToWorld.Rotator();
	}
	else
	{
		DesiredView.Location = GetComponentLocation();
		DesiredView.Rotation = GetComponentRotation();
	}

	DesiredView.FOV = bUseAdditiveOffset ? (FieldOfView + AdditiveFOVOffset) : FieldOfView;
	DesiredView.AspectRatio = AspectRatio;
	DesiredView.bConstrainAspectRatio = bConstrainAspectRatio;
	DesiredView.bUseFieldOfViewForLOD = bUseFieldOfViewForLOD;
	DesiredView.ProjectionMode = ProjectionMode;
	DesiredView.OrthoWidth = OrthoWidth;
	DesiredView.OrthoNearClipPlane = OrthoNearClipPlane;
	DesiredView.OrthoFarClipPlane = OrthoFarClipPlane;
	DesiredView.bAutoCalculateOrthoPlanes = bAutoCalculateOrthoPlanes;
	DesiredView.AutoPlaneShift = AutoPlaneShift;
	DesiredView.bUpdateOrthoPlanes = bUpdateOrthoPlanes;
	DesiredView.bUseCameraHeightAsViewTarget = bUseCameraHeightAsViewTarget;
	DesiredView.FirstPersonFOV = bEnableFirstPersonFieldOfView ? FirstPersonFieldOfView : DesiredView.FOV;
	DesiredView.FirstPersonScale = bEnableFirstPersonScale ? FirstPersonScale : 1.0f;
	DesiredView.bUseFirstPersonParameters = bEnableFirstPersonFieldOfView || bEnableFirstPersonScale;
	
	DesiredView.ApplyOverscan(Overscan, bScaleResolutionWithOverscan, bCropOverscan);
	DesiredView.ApplyAsymmetricOverscan(AsymmetricOverscan, bScaleResolutionWithOverscan, bCropOverscan);
	
	if (bAutoCalculateOrthoPlanes)
	{
		if (const AActor* ViewTarget = GetOwner())
		{
			DesiredView.SetCameraToViewTarget(ViewTarget->GetActorLocation());
		}
	}

	if (bOverrideAspectRatioAxisConstraint)
	{
		DesiredView.AspectRatioAxisConstraint = AspectRatioAxisConstraint;
	}

	// See if the CameraActor wants to override the PostProcess settings used.
	DesiredView.PostProcessBlendWeight = PostProcessBlendWeight;
	if (PostProcessBlendWeight > 0.0f)
	{
		DesiredView.PostProcessSettings = PostProcessSettings;
	}

	// If this camera component has a motion vector simumlation transform, use that for the current view's previous transform
	DesiredView.PreviousViewTransform = FMotionVectorSimulation::Get().GetPreviousTransform(this);
}

#if WITH_EDITOR
void UCameraComponent::CheckForErrors()
{
	Super::CheckForErrors();

	if (AspectRatio <= 0.0f)
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT( "MapCheck_Message_CameraAspectRatioIsZero", "Camera has AspectRatio=0 - please set this to something non-zero" ) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::CameraAspectRatioIsZero));
	}
}

bool UCameraComponent::GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut)
{
	if (IsActive())
	{
		GetCameraView(DeltaTime, ViewOut);
	}
	return IsActive();
}
#endif	// WITH_EDITOR

void UCameraComponent::NotifyCameraCut()
{
	// if we are owned by a camera actor, notify it too
	// note: many camera components are not part of camera actors, so notification should begin at the
	// component level.
	ACameraActor* const OwningCamera = Cast<ACameraActor>(GetOwner());
	if (OwningCamera)
	{
		OwningCamera->NotifyCameraCut();
	}
};

void UCameraComponent::AddAdditiveOffset(FTransform const& Transform, float FOV)
{
	bUseAdditiveOffset = true;
	AdditiveOffset = AdditiveOffset * Transform;
	AdditiveFOVOffset += FOV;

#if WITH_EDITORONLY_DATA
	UpdateProxyMeshTransform();
#endif
}

/** Removes any additive offset. */
void UCameraComponent::ClearAdditiveOffset()
{
	bUseAdditiveOffset = false;
	AdditiveOffset = FTransform::Identity;
	AdditiveFOVOffset = 0.f;

#if WITH_EDITORONLY_DATA
	UpdateProxyMeshTransform();
#endif
}

void UCameraComponent::GetAdditiveOffset(FTransform& OutAdditiveOffset, float& OutAdditiveFOVOffset) const
{
	OutAdditiveOffset = AdditiveOffset;
	OutAdditiveFOVOffset = AdditiveFOVOffset;
}

void UCameraComponent::AddExtraPostProcessBlend(FPostProcessSettings const& PPSettings, float PPBlendWeight)
{
	checkSlow(ExtraPostProcessBlends.Num() == ExtraPostProcessBlendWeights.Num());
	ExtraPostProcessBlends.Add(PPSettings);
	ExtraPostProcessBlendWeights.Add(PPBlendWeight);
}

void UCameraComponent::ClearExtraPostProcessBlends()
{
	ExtraPostProcessBlends.Empty();
	ExtraPostProcessBlendWeights.Empty();
}

void UCameraComponent::GetExtraPostProcessBlends(TArray<FPostProcessSettings>& OutSettings, TArray<float>& OutWeights) const
{
	OutSettings = ExtraPostProcessBlends;
	OutWeights = ExtraPostProcessBlendWeights;
}


#undef LOCTEXT_NAMESPACE


