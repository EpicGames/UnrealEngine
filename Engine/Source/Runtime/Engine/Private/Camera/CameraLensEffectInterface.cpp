// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraLensEffectInterface.h"

#include "GameFramework/Actor.h"
#include "Logging/MessageLog.h"
#include "Particles/ParticleSystemComponent.h"
#include "UObject/ScriptInterface.h"
#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraLensEffectInterface)

namespace CameraLensEffectCVars
{
	static bool bAttachedEmitterTransformOptimizationEnabled = true;
	static FAutoConsoleVariableRef CVarAttachedEmitterTransformOptimizationEnabled(
		TEXT("CameraLensEffect.AttachedEmitterTransformOptimizationEnabled"),
		bAttachedEmitterTransformOptimizationEnabled,
		TEXT("If true, GetAttachedEmitterTransform avoids doing unnecessary work when the camera FOV matches the emitter's base FOV, and avoids recalculating constant values."),
		ECVF_Default);
}

TArray<UFXSystemComponent*> ICameraLensEffectInterface::GetParticleComponents() const
{
	const AActor* const ConstActorThis = Cast<AActor>(this);
	AActor* const ActorThis = const_cast<AActor*>(ConstActorThis);
	TArray<UFXSystemComponent*> ParticleComps;
	ActorThis->GetComponents(ParticleComps, true);
	return ParticleComps;
}

UFXSystemComponent* ICameraLensEffectInterface::GetPrimaryParticleComponent() const
{
	TArray<UFXSystemComponent*> AllComps = GetParticleComponents();

	return AllComps.Num() != 0 ? AllComps[0] : nullptr;
}

void ICameraLensEffectInterface::UpdateLocation(const FVector& CamLoc, const FRotator& CamRot, float CamFOVDeg)
{
	if (AActor* ActorThis = Cast<AActor>(this))
	{
		ActorThis->SetActorTransform(GetAttachedEmitterTransform(ActorThis, CamLoc, CamRot, CamFOVDeg));
	}
}

FTransform ICameraLensEffectInterface::GetAttachedEmitterTransform(const AActor* Emitter, const FVector& CamLoc, const FRotator& CamRot, float CamFOVDeg)
{
	if (const ICameraLensEffectInterface* InterfaceEmitter = Cast<ICameraLensEffectInterface>(Emitter))
	{
		// adjust for FOV
		// base dist uses BaseFOV which is set on the indiv camera lens effect class
		FTransform RelativeTransformAdjustedForFOV = InterfaceEmitter->GetRelativeTransform();
		FVector AdjustedRelativeLoc = RelativeTransformAdjustedForFOV.GetLocation();

		const float BaseFOV = InterfaceEmitter->GetBaseFOV();
		if (!CameraLensEffectCVars::bAttachedEmitterTransformOptimizationEnabled || !FMath::IsNearlyEqual(BaseFOV, CamFOVDeg))
		{
			constexpr float HalfFovDegreesToRadians = 0.5f * UE_PI / 180.f;
			AdjustedRelativeLoc.X *= FMath::Tan(BaseFOV * HalfFovDegreesToRadians) / FMath::Tan(CamFOVDeg * HalfFovDegreesToRadians);
			RelativeTransformAdjustedForFOV.SetLocation(AdjustedRelativeLoc);
		}

		FTransform const CameraToWorld(CamRot, CamLoc);

		// RelativeTransform is "effect to camera"
		FTransform const EffectToWorld = RelativeTransformAdjustedForFOV * CameraToWorld;

		return EffectToWorld;
	}

	return FTransform::Identity;
}

TSubclassOf<AActor> UCameraLensEffectInterfaceClassSupportLibrary::GetInterfaceClass(const FCameraLensInterfaceClassSupport& CameraLens)
{
	if (CameraLens.Class && CameraLens.Class->ImplementsInterface(UCameraLensEffectInterface::StaticClass()))
	{
		return CameraLens.Class;
	}

	if (CameraLens.Class)
	{
		FText ConstructedText = FText::Format(NSLOCTEXT("CameraLensEffectInterfaceClassSupportLibrary", "GetInterfaceClass_ClassSetWithoutInterface", "{0} found within wrapper passed to UCameraLensEffectInterfaceClassSupportLibrary::GetInterfaceClass, but does not implement CameraLensEffectInterface!"), FText::FromString(CameraLens.Class->GetName()));
		FFrame::KismetExecutionMessage(*ConstructedText.ToString(), ELogVerbosity::Error, TEXT("CameraLensEffectInterfaceClassSupportLibraryxxGetInterfaceClass_ClassSetWithoutInterface"));
		FMessageLog("PIE").Error(ConstructedText);
	}

	// could technically clear out the class field if we get here, but technically violates const intent of the API and I don't want to change to non-const, and there's no way to circumvent the checks in place ANYWAY.
	return nullptr;
}

void UCameraLensEffectInterfaceClassSupportLibrary::IsInterfaceClassValid(const FCameraLensInterfaceClassSupport& CameraLens, EInterfaceValidResult& Result)
{
	// note: GetInterfaceClass surfaces errors - no need to do that ourselves!
	if (TSubclassOf<AActor> Check = GetInterfaceClass(CameraLens))
	{
		Result = EInterfaceValidResult::Valid;
	}
	Result = EInterfaceValidResult::Invalid;
}

void UCameraLensEffectInterfaceClassSupportLibrary::IsInterfaceValid(const TScriptInterface<ICameraLensEffectInterface>& CameraLens, EInterfaceValidResult& Result)
{
	// TScriptInterface operator bool will conduct implement checks
	if (CameraLens)
	{
		Result = EInterfaceValidResult::Valid;
	}
	Result = EInterfaceValidResult::Invalid;
}

void UCameraLensEffectInterfaceClassSupportLibrary::SetInterfaceClass(TSubclassOf<AActor> Class, FCameraLensInterfaceClassSupport& Var, EInterfaceValidResult& Result)
{
	if (Class && Class->ImplementsInterface(UCameraLensEffectInterface::StaticClass()))
	{
		Var.Class = Class;
		Result = EInterfaceValidResult::Valid;
	}
	else
	{
		Result = EInterfaceValidResult::Invalid;

		if (Class)
		{
			FText ConstructedText = FText::Format(NSLOCTEXT("CameraLensEffectInterfaceClassSupportLibrary", "SetInterfaceClass_ClassSetWithoutInterface", "{0} passed to UCameraLensEffectInterfaceClassSupportLibrary::SetInterfaceClass, but does not implement CameraLensEffectInterface!"), FText::FromString(GetNameSafe(Class)));
			FFrame::KismetExecutionMessage(*ConstructedText.ToString(), ELogVerbosity::Error, TEXT("CameraLensEffectInterfaceClassSupportLibraryxxSetInterfaceClass_ClassSetWithoutInterface"));
			FMessageLog("PIE").Error(ConstructedText);
		}
	}
}

