// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraModifier.h"
#include "Camera/PlayerCameraManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraModifier)

//////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY_STATIC(LogCamera, Log, All);

//////////////////////////////////////////////////////////////////////////
// UCameraModifier

UCameraModifier::UCameraModifier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Priority = 127;
}

bool UCameraModifier::ModifyCamera(float DeltaTime, FMinimalViewInfo& InOutPOV)
{
	// Update the alpha
	UpdateAlpha(DeltaTime);

	// let native do what it wants
	ModifyCamera(DeltaTime, InOutPOV.Location, InOutPOV.Rotation, InOutPOV.FOV, InOutPOV.Location, InOutPOV.Rotation, InOutPOV.FOV);
	// let BP do what it wants
	BlueprintModifyCamera(DeltaTime, InOutPOV.Location, InOutPOV.Rotation, InOutPOV.FOV, InOutPOV.Location, InOutPOV.Rotation, InOutPOV.FOV);

	if (CameraOwner)
	{
		// note: pushing these through the cached PP blend system in the camera to get
		// proper layered blending, rather than letting subsequent mods stomp over each other in the 
		// InOutPOV struct.
		{
			float PPBlendWeight = 0.f;
			FPostProcessSettings PPSettings;
			
			//  Let native code modify the post process settings.
			ModifyPostProcess(DeltaTime, PPBlendWeight, PPSettings);
			
			// Allow blueprint to modify the post process settings.
			BlueprintModifyPostProcess(DeltaTime, PPBlendWeight, PPSettings);
			
			if (PPBlendWeight > 0.f)
			{
				CameraOwner->AddCachedPPBlend(PPSettings, PPBlendWeight);
			}
		}
	}

	// If pending disable and fully alpha'd out, truly disable this modifier
	if (bPendingDisable && (Alpha <= 0.f))
	{
		DisableModifier(true);
	}

	// allow subsequent modifiers to update
	return false;
}

float UCameraModifier::GetTargetAlpha()
{
	return bPendingDisable ? 0.0f : 1.f;
}

void UCameraModifier::UpdateAlpha(float DeltaTime)
{
	float const TargetAlpha = GetTargetAlpha();
	float const BlendTime = (TargetAlpha == 0.f) ? AlphaOutTime : AlphaInTime;

	// interpolate!
	if (BlendTime <= 0.f)
	{
		// no blendtime means no blending, just go directly to target alpha
		Alpha = TargetAlpha;
	}
	else if (Alpha > TargetAlpha)
	{
		// interpolate downward to target, while protecting against overshooting
		Alpha = FMath::Max<float>(Alpha - DeltaTime / BlendTime, TargetAlpha);
	}
	else
	{
		// interpolate upward to target, while protecting against overshooting
		Alpha = FMath::Min<float>(Alpha + DeltaTime / BlendTime, TargetAlpha);
	}
}

bool UCameraModifier::IsDisabled() const
{
	return bDisabled;
}

bool UCameraModifier::IsPendingDisable() const
{
	return bPendingDisable;
}

AActor* UCameraModifier::GetViewTarget() const
{
	return CameraOwner ? CameraOwner->GetViewTarget() : nullptr;
}

void UCameraModifier::AddedToCamera( APlayerCameraManager* Camera ) 
{
	if (CameraOwner)
	{
		CameraOwner->OnDestroyed.RemoveDynamic(this, &UCameraModifier::OnCameraOwnerDestroyed);
	}
	CameraOwner = Camera;
	if (CameraOwner)
	{
		CameraOwner->OnDestroyed.AddDynamic(this, &UCameraModifier::OnCameraOwnerDestroyed);
	}
}

void UCameraModifier::OnCameraOwnerDestroyed(AActor* InOwner)
{
	if (InOwner == CameraOwner)
	{
		CameraOwner = nullptr;
	}
}

UWorld* UCameraModifier::GetWorld() const
{
	return CameraOwner ? CameraOwner->GetWorld() : nullptr;
}

void UCameraModifier::DisableModifier(bool bImmediate)
{
	if (bImmediate)
	{
		bDisabled = true;
		bPendingDisable = false;
	}
	else if (!bDisabled)
	{
		bPendingDisable = true;
	}
}

void UCameraModifier::EnableModifier()
{
	bDisabled = false;
	bPendingDisable = false;
}

void UCameraModifier::ToggleModifier()
{
	if( bDisabled )
	{
		EnableModifier();
	}
	else
	{
		DisableModifier();
	}
}

bool UCameraModifier::ProcessViewRotation( AActor* ViewTarget, float DeltaTime, FRotator& OutViewRotation, FRotator& OutDeltaRot)
{
	return false;
}

void UCameraModifier::ModifyCamera(float DeltaTime, FVector ViewLocation, FRotator ViewRotation, float FOV, FVector& NewViewLocation, FRotator& NewViewRotation, float& NewFOV)
{

}

void UCameraModifier::ModifyPostProcess(float DeltaTime, float& PostProcessBlendWeight, FPostProcessSettings& PostProcessSettings)
{

}

void UCameraModifier::DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{

}

