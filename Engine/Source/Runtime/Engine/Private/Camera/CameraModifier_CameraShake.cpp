// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraModifier_CameraShake.h"
#include "BatchedElements.h"
#include "Camera/CameraShakeBase.h"
#include "Camera/CameraShakeSourceComponent.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/PlayerController.h"
#include "HAL/IConsoleManager.h"
#include "Stats/Stats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraModifier_CameraShake)

DEFINE_LOG_CATEGORY_STATIC(LogCameraShake, Warning, All);

//////////////////////////////////////////////////////////////////////////
// UCameraModifier_CameraShake

DECLARE_CYCLE_STAT(TEXT("AddCameraShake"), STAT_AddCameraShake, STATGROUP_Game);

TAutoConsoleVariable<bool> GShowCameraShakeDebugCVar(
	TEXT("r.CameraShakeDebug"),
	false,
	TEXT("Show extra debug info for camera shakes (requires `showdebug CAMERA`)"));
TAutoConsoleVariable<bool> GShowCameraShakeDebugLocationCVar(
	TEXT("r.CameraShakeDebug.Location"),
	true,
	TEXT("Whether to show camera shakes' location modifications (defaults to true)"));
TAutoConsoleVariable<bool> GShowCameraShakeDebugRotationCVar(
	TEXT("r.CameraShakeDebug.Rotation"),
	true,
	TEXT("Whether to show camera shakes' rotation modifications (defaults to true)"));
TAutoConsoleVariable<bool> GCameraShakeDebugLargeGraphCVar(
	TEXT("r.CameraShakeDebug.LargeGraph"),
	false,
	TEXT("Draws larger graphs for camera shake debug info"));
TAutoConsoleVariable<float> GCameraShakeDebugInfoRecordLimitCVar(
	TEXT("r.CameraShakeDebug.InfoRecordLimit"),
	2,
	TEXT("How many seconds to keep while recording camera shake debug info (defaults to 2 seconds)"));

UCameraModifier_CameraShake::UCameraModifier_CameraShake(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if UE_ENABLE_DEBUG_DRAWING
	, bRecordDebugData(false)
#endif
{
	SplitScreenShakeScale = 0.5f;
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddUObject(this, &UCameraModifier_CameraShake::OnPreGarbageCollect);
}

void UCameraModifier_CameraShake::BeginDestroy()
{
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
	Super::BeginDestroy();
}

void UCameraModifier_CameraShake::OnPreGarbageCollect()
{
	RemoveInvalidObjectsFromExpiredPool();
}

bool UCameraModifier_CameraShake::ModifyCamera(float DeltaTime, FMinimalViewInfo& InOutPOV)
{
	// Call super where modifier may be disabled
	Super::ModifyCamera(DeltaTime, InOutPOV);

	// If no alpha, exit early
	if( Alpha <= 0.f )
	{
		return false;
	}

#if UE_ENABLE_DEBUG_DRAWING
	const float DebugDataLimit = GCameraShakeDebugInfoRecordLimitCVar.GetValueOnGameThread();
#endif

	// Update and apply active shakes
	if( ActiveShakes.Num() > 0 )
	{
		for (FActiveCameraShakeInfo& ShakeInfo : ActiveShakes)
		{
			if (ShakeInfo.ShakeInstance != nullptr)
			{
#if UE_ENABLE_DEBUG_DRAWING
				const FVector OrigLocation = InOutPOV.Location;
				const FRotator OrigRotation = InOutPOV.Rotation;
#endif

				// Compute the scale of this shake for this frame according to the location
				// of its source.
				float CurShakeAlpha = Alpha;
				if (ShakeInfo.ShakeSource.IsValid())
				{
					const UCameraShakeSourceComponent* SourceComponent = ShakeInfo.ShakeSource.Get();
					const float AttenuationFactor = SourceComponent->GetAttenuationFactor(InOutPOV.Location);
					CurShakeAlpha *= AttenuationFactor;
				}

				ShakeInfo.ShakeInstance->UpdateAndApplyCameraShake(DeltaTime, CurShakeAlpha, InOutPOV);

#if UE_ENABLE_DEBUG_DRAWING
				if (bRecordDebugData && ShakeInfo.DebugDataIndex != INDEX_NONE)
				{
					// Record the delta between the original camera location/rotation, and the result
					// of this camera shake.
					check(DebugShakes.IsValidIndex(ShakeInfo.DebugDataIndex));
					FCameraShakeDebugData& DebugShake = DebugShakes[ShakeInfo.DebugDataIndex];
					const float PreviousAccumulatedTime = (DebugShake.DataPoints.Num() > 0) ?
						DebugShake.DataPoints.Last().AccumulatedTime : 0.f;

					const FVector DeltaLocation = InOutPOV.Location - OrigLocation;
					const FRotator DeltaRotation = (InOutPOV.Rotation - OrigRotation).GetNormalized();
					const float AccumulatedTime = PreviousAccumulatedTime + DeltaTime;
					const FCameraShakeDebugDataPoint FrameData { AccumulatedTime, DeltaLocation, DeltaRotation };
					DebugShake.DataPoints.Add(FrameData);
				}
#endif
			}
		}

		// Delete any obsolete shakes
		for (int32 i = ActiveShakes.Num() - 1; i >= 0; i--)
		{
			FActiveCameraShakeInfo ShakeInfo = ActiveShakes[i]; // Copy struct, we're going to maybe delete it.
			if (ShakeInfo.ShakeInstance == nullptr || ShakeInfo.ShakeInstance->IsFinished() || ShakeInfo.ShakeSource.IsStale())
			{
				if (ShakeInfo.ShakeInstance != nullptr)
				{
					ShakeInfo.ShakeInstance->TeardownShake();
				}

				ActiveShakes.RemoveAt(i, 1);
				UE_LOGF(LogCameraShake, Verbose, "UCameraModifier_CameraShake::ModifyCamera Removing obsolete shake %ls", *GetNameSafe(ShakeInfo.ShakeInstance));

				SaveShakeInExpiredPoolIfPossible(ShakeInfo);
			}
		}
	}

#if UE_ENABLE_DEBUG_DRAWING
	// Add empty data points for the inactive shakes, and get rid of those who expired.
	for (int32 Index = DebugShakes.Num() - 1; Index >= 0; --Index)
	{
		FCameraShakeDebugData& DebugShake = DebugShakes[Index];
		if (DebugShake.bIsInactive)
		{
			DebugShake.TimeInactive += DeltaTime;
			if (DebugShake.TimeInactive >= DebugDataLimit)
			{
				// We can remove this debug data, it's been kept in the debug display long enough.
				DebugShakes.RemoveAt(Index);

				// Update the debug indices for active shakes.
				for (FActiveCameraShakeInfo& ShakeInfo : ActiveShakes)
				{
					check(ShakeInfo.DebugDataIndex != Index);
					if (ShakeInfo.DebugDataIndex > Index)
					{
						--ShakeInfo.DebugDataIndex;
					}
				}
			}
			else
			{
				// Add empty data points for a while.
				const float PreviousAccumulatedTime = (DebugShake.DataPoints.Num() > 0) ?
					DebugShake.DataPoints.Last().AccumulatedTime : 0.f;
				const float AccumulatedTime = PreviousAccumulatedTime + DeltaTime;
				const FCameraShakeDebugDataPoint FrameData{ AccumulatedTime, FVector::ZeroVector, FRotator::ZeroRotator };
				DebugShake.DataPoints.Add(FrameData);
			}
		}
	}

	// Remove old data points.
	for (FCameraShakeDebugData& DebugShake : DebugShakes)
	{
		if (DebugShake.DataPoints.Num() > 0)
		{
			const float CutoffTime = DebugShake.DataPoints.Last().AccumulatedTime - DebugDataLimit;
			for (int32 Index = 0; Index < DebugShake.DataPoints.Num(); ++Index)
			{
				if (DebugShake.DataPoints[Index].AccumulatedTime > CutoffTime)
				{
					DebugShake.DataPoints.RemoveAt(0, Index);
					break;
				}
			}
		}
	}
#endif

	// If ModifyCamera returns true, exit loop
	// Allows high priority things to dictate if they are
	// the last modifier to be applied
	// Returning true causes to stop adding another modifier! 
	// Returning false is the right behavior since this is not high priority modifier.
	return false;
}

UCameraShakeBase* UCameraModifier_CameraShake::AddCameraShake(TSubclassOf<UCameraShakeBase> ShakeClass, const FAddCameraShakeParams& Params)
{
	SCOPE_CYCLE_COUNTER(STAT_AddCameraShake);

	UE_LOGF(LogCameraShake, Verbose, "UCameraModifier_CameraShake::AddCameraShake %ls", *GetNameSafe(ShakeClass));

	if (ShakeClass != nullptr)
	{
		float Scale = Params.Scale;
		const UCameraShakeSourceComponent* SourceComponent = Params.SourceComponent;
		const bool bIsCustomInitialized = Params.Initializer.IsBound();

		// Adjust for splitscreen
		if (CameraOwner != nullptr && CameraOwner->PCOwner != nullptr)
		{
			const ULocalPlayer* LocalPlayer = CameraOwner->PCOwner->GetLocalPlayer();
			if (LocalPlayer != nullptr && LocalPlayer->ViewportClient != nullptr)
			{
				if (LocalPlayer->ViewportClient->GetCurrentSplitscreenConfiguration() != ESplitScreenType::None)
				{
					Scale *= SplitScreenShakeScale;
				}
			}
		}

		UCameraShakeBase const* const ShakeCDO = GetDefault<UCameraShakeBase>(ShakeClass);
		const bool bIsSingleInstance = ShakeCDO && ShakeCDO->bSingleInstance;
		if (bIsSingleInstance)
		{
			// Look for existing instance of same class
			for (FActiveCameraShakeInfo& ShakeInfo : ActiveShakes)
			{
				UCameraShakeBase* ShakeInst = ShakeInfo.ShakeInstance;
				if (ShakeInst && (ShakeClass == ShakeInst->GetClass()))
				{
					if (!ShakeInfo.bIsCustomInitialized && !bIsCustomInitialized)
					{
						// Just restart the existing shake, possibly at the new location.
						// Warning: if the shake source changes, this would "teleport" the shake, which might create a visual
						// artifact, if the user didn't intend to do this.
						ShakeInfo.ShakeSource = SourceComponent;
						ShakeInst->StartShake(CameraOwner, Scale, Params.PlaySpace, Params.UserPlaySpaceRot);
						return ShakeInst;
					}
					else
					{
						// If either the old or new shake are custom initialized, we can't
						// reliably restart the existing shake and expect it to be the same as what the caller wants. 
						// So we forcibly stop the existing shake immediately and will create a brand new one.
						ShakeInst->StopShake(true);
						ShakeInst->TeardownShake();
						// Discard it right away so the spot is free in the active shakes array.
						ShakeInfo.ShakeInstance = nullptr;
					}
				}
			}
		}

		// Try to find a shake in the expired pool
		UCameraShakeBase* NewInst = ReclaimShakeFromExpiredPool(ShakeClass);

		// No old shakes, create a new one
		if (NewInst == nullptr)
		{
			NewInst = NewObject<UCameraShakeBase>(this, ShakeClass);
		}

		if (NewInst)
		{
			// Custom initialization if necessary.
			if (bIsCustomInitialized)
			{
				Params.Initializer.Execute(NewInst);
			}

			// Initialize new shake and add it to the list of active shakes
			FCameraShakeBaseStartParams StartParams;
			StartParams.CameraManager = CameraOwner;
			StartParams.Scale = Scale;
			StartParams.PlaySpace = Params.PlaySpace;
			StartParams.UserPlaySpaceRot = Params.UserPlaySpaceRot;
			StartParams.DurationOverride = Params.DurationOverride;
			NewInst->StartShake(StartParams);

			// Look for nulls in the array to replace first -- keeps the array compact
			bool bReplacedNull = false;
			for (int32 Idx = 0; Idx < ActiveShakes.Num(); ++Idx)
			{
				FActiveCameraShakeInfo& ShakeInfo = ActiveShakes[Idx];
				if (ShakeInfo.ShakeInstance == nullptr)
				{
					ShakeInfo.ShakeInstance = NewInst;
					ShakeInfo.ShakeSource = SourceComponent;
					ShakeInfo.bIsCustomInitialized = bIsCustomInitialized;
					bReplacedNull = true;

#if UE_ENABLE_DEBUG_DRAWING
					AddCameraShakeDebugData(ShakeInfo);
#endif
				}
			}

			// no holes, extend the array
			if (bReplacedNull == false)
			{
				FActiveCameraShakeInfo ShakeInfo;
				ShakeInfo.ShakeInstance = NewInst;
				ShakeInfo.ShakeSource = SourceComponent;
				ShakeInfo.bIsCustomInitialized = bIsCustomInitialized;
				ActiveShakes.Emplace(ShakeInfo);
				UE_LOGF(LogCameraShake, Verbose, "UCameraModifier_CameraShake::AddCameraShake %ls Active Instance Added", *GetNameSafe(ShakeInfo.ShakeInstance));

#if UE_ENABLE_DEBUG_DRAWING
				AddCameraShakeDebugData(ActiveShakes.Last());
#endif
			}
		}

		return NewInst;
	}

	return nullptr;
}

void UCameraModifier_CameraShake::SaveShakeInExpiredPool(UCameraShakeBase* ShakeInst)
{
	FPooledCameraShakes& PooledCameraShakes = ExpiredPooledShakesMap.FindOrAdd(ShakeInst->GetClass());
	if (PooledCameraShakes.PooledShakes.Num() < 5)
	{
		PooledCameraShakes.PooledShakes.Emplace(ShakeInst);
	}
}

void UCameraModifier_CameraShake::SaveShakeInExpiredPoolIfPossible(const FActiveCameraShakeInfo& ShakeInfo)
{
	if (ShakeInfo.ShakeInstance && !ShakeInfo.bIsCustomInitialized)
	{
		SaveShakeInExpiredPool(ShakeInfo.ShakeInstance);
	}

#if UE_ENABLE_DEBUG_DRAWING
	RemoveCameraShakeDebugData(ShakeInfo);
#endif
}

UCameraShakeBase* UCameraModifier_CameraShake::ReclaimShakeFromExpiredPool(TSubclassOf<UCameraShakeBase> CameraShakeClass)
{
	if (FPooledCameraShakes* PooledCameraShakes = ExpiredPooledShakesMap.Find(CameraShakeClass))
	{
		if (PooledCameraShakes->PooledShakes.Num() > 0)
		{
			UCameraShakeBase* OldShake = PooledCameraShakes->PooledShakes.Pop();
			// Calling new object with the exact same name will re-initialize the uobject in place
			OldShake = NewObject<UCameraShakeBase>(this, CameraShakeClass, OldShake->GetFName());
			return OldShake;
		}
	}
	return nullptr;
}

void UCameraModifier_CameraShake::RemoveInvalidObjectsFromExpiredPool()
{
	for (auto CamShakeItr = ExpiredPooledShakesMap.CreateIterator(); CamShakeItr; ++CamShakeItr)
	{
		if (!IsValid(CamShakeItr.Key()))
		{
			CamShakeItr.RemoveCurrent();
			continue;
		}
		TArray<TObjectPtr<UCameraShakeBase>>& PooledShakes = CamShakeItr.Value().PooledShakes;
		for (int32 i = PooledShakes.Num() - 1; i >= 0; --i)
		{
			if (!IsValid(PooledShakes[i]))
			{
				PooledShakes.RemoveAtSwap(i);
			}
		}
	}
}

void UCameraModifier_CameraShake::GetActiveCameraShakes(TArray<FActiveCameraShakeInfo>& ActiveCameraShakes) const
{
	ActiveCameraShakes.Append(ActiveShakes);
}

void UCameraModifier_CameraShake::RemoveCameraShake(UCameraShakeBase* ShakeInst, bool bImmediately)
{
	for (int32 i = 0; i < ActiveShakes.Num(); ++i)
	{
		FActiveCameraShakeInfo& ShakeInfo = ActiveShakes[i];
		if (ShakeInfo.ShakeInstance == ShakeInst)
		{
			ShakeInst->StopShake(bImmediately);
			if (bImmediately)
			{
				ShakeInst->TeardownShake();
				SaveShakeInExpiredPoolIfPossible(ShakeInfo);
				ActiveShakes.RemoveAt(i, 1);
				UE_LOGF(LogCameraShake, Verbose, "UCameraModifier_CameraShake::RemoveCameraShake %ls", *GetNameSafe(ShakeInfo.ShakeInstance));
			}
			break;
		}
	}
}

void UCameraModifier_CameraShake::RemoveAllCameraShakesOfClass(TSubclassOf<UCameraShakeBase> ShakeClass, bool bImmediately)
{
	for (int32 i = ActiveShakes.Num()- 1; i >= 0; --i)
	{
		FActiveCameraShakeInfo& ShakeInfo = ActiveShakes[i];
		UCameraShakeBase* ShakeInst = ShakeInfo.ShakeInstance;
		if (ShakeInst != nullptr && (ShakeInst->GetClass()->IsChildOf(ShakeClass)))
		{
			ShakeInst->StopShake(bImmediately);
			if (bImmediately)
			{
				ShakeInst->TeardownShake();
				SaveShakeInExpiredPoolIfPossible(ShakeInfo);
				ActiveShakes.RemoveAt(i, 1);
				UE_LOGF(LogCameraShake, Verbose, "UCameraModifier_CameraShake::RemoveAllCameraShakesOfClass %ls", *GetNameSafe(ShakeInfo.ShakeInstance));
			}
		}
	}
}

void UCameraModifier_CameraShake::RemoveAllCameraShakesFromSource(const UCameraShakeSourceComponent* SourceComponent, bool bImmediately)
{
	for (int32 i = ActiveShakes.Num() - 1; i >= 0; --i)
	{
		FActiveCameraShakeInfo& ShakeInfo = ActiveShakes[i];
		if (ShakeInfo.ShakeSource.Get() == SourceComponent && ShakeInfo.ShakeInstance != nullptr)
		{
			ShakeInfo.ShakeInstance->StopShake(bImmediately);
			if (bImmediately)
			{
				ShakeInfo.ShakeInstance->TeardownShake();
				SaveShakeInExpiredPoolIfPossible(ShakeInfo);
				ActiveShakes.RemoveAt(i, 1);
				UE_LOGF(LogCameraShake, Verbose, "UCameraModifier_CameraShake::RemoveAllCameraShakesFromSource %ls", *GetNameSafe(ShakeInfo.ShakeInstance));
			}
		}
	}
}

void UCameraModifier_CameraShake::RemoveAllCameraShakesOfClassFromSource(TSubclassOf<UCameraShakeBase> ShakeClass, const UCameraShakeSourceComponent* SourceComponent, bool bImmediately)
{
	for (int32 i = ActiveShakes.Num() - 1; i >= 0; --i)
	{
		FActiveCameraShakeInfo& ShakeInfo = ActiveShakes[i];
		if (ShakeInfo.ShakeSource.Get() == SourceComponent && 
				ShakeInfo.ShakeInstance != nullptr && 
				ShakeInfo.ShakeInstance->GetClass()->IsChildOf(ShakeClass))
		{
			ShakeInfo.ShakeInstance->StopShake(bImmediately);
			if (bImmediately)
			{
				ShakeInfo.ShakeInstance->TeardownShake();
				SaveShakeInExpiredPoolIfPossible(ShakeInfo);
				ActiveShakes.RemoveAt(i, 1);
				UE_LOGF(LogCameraShake, Verbose, "UCameraModifier_CameraShake::RemoveAllCameraShakesOfClassFromSource %ls", *GetNameSafe(ShakeInfo.ShakeInstance));
			}
		}
	}
}

void UCameraModifier_CameraShake::RemoveAllCameraShakes(bool bImmediately)
{
	// Clean up any active camera shake anims
	for (FActiveCameraShakeInfo& ShakeInfo : ActiveShakes)
	{
		ShakeInfo.ShakeInstance->StopShake(bImmediately);
	}

	if (bImmediately)
	{
		for (FActiveCameraShakeInfo& ShakeInfo : ActiveShakes)
		{
			ShakeInfo.ShakeInstance->TeardownShake();
			SaveShakeInExpiredPoolIfPossible(ShakeInfo);
		}

		// Clear ActiveShakes array
		ActiveShakes.Empty();
	}
}

void UCameraModifier_CameraShake::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	Canvas->SetDrawColor(FColor::Yellow);
	UFont* DrawFont = GEngine->GetSmallFont();

	int Indentation = 1;
	int LineNumber = FMath::CeilToInt(YPos / YL);

	Canvas->DrawText(DrawFont, FString::Printf(TEXT("Modifier_CameraShake %s, Alpha:%f"), *GetNameSafe(this), Alpha), Indentation * YL, (LineNumber++) * YL);

	Indentation = 2;
	for (int i = 0; i < ActiveShakes.Num(); i++)
	{
		FActiveCameraShakeInfo& ShakeInfo = ActiveShakes[i];

		if (ShakeInfo.ShakeInstance != nullptr)
		{
			const FString DurationString = !ShakeInfo.ShakeInstance->GetCameraShakeDuration().IsInfinite() ? 
				FString::SanitizeFloat(ShakeInfo.ShakeInstance->GetCameraShakeDuration().Get()) : TEXT("Infinite");
			Canvas->DrawText(DrawFont,
					FString::Printf(TEXT("[%d] %s Source:%s"), 
						i, *GetNameSafe(ShakeInfo.ShakeInstance), *GetNameSafe(ShakeInfo.ShakeSource.Get())), 
					Indentation * YL, (LineNumber++) * YL);
		}
	}

	YPos = LineNumber * YL;

#if UE_ENABLE_DEBUG_DRAWING
	DisplayDebugGraphs(Canvas, DebugDisplay);
#endif

	Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);
}

#if UE_ENABLE_DEBUG_DRAWING

void UCameraModifier_CameraShake::AddCameraShakeDebugData(FActiveCameraShakeInfo& ShakeInfo)
{
	FString ShakeInstanceName = GetNameSafe(ShakeInfo.ShakeInstance);
	TSubclassOf<UCameraShakeBase> ShakeClass = ShakeInfo.ShakeInstance->GetClass();

	for (int32 Index = 0; Index < DebugShakes.Num(); ++Index)
	{
		FCameraShakeDebugData& DebugShake = DebugShakes[Index];

		if (DebugShake.bIsInactive && 
				DebugShake.ShakeClass == ShakeClass && 
				DebugShake.ShakeInstanceName == ShakeInstanceName)
		{
			// Found a matching debug info, re-use it.
			ShakeInfo.DebugDataIndex = Index;
			DebugShake.bIsInactive = false;
			DebugShake.TimeInactive = 0.f;
			return;
		}
	}

	FCameraShakeDebugData NewDebugShake;
	NewDebugShake.ShakeClass = ShakeClass;
	NewDebugShake.ShakeInstanceName = ShakeInstanceName;
	ShakeInfo.DebugDataIndex = DebugShakes.Emplace(NewDebugShake);
}

void UCameraModifier_CameraShake::RemoveCameraShakeDebugData(const FActiveCameraShakeInfo& ShakeInfo)
{
	if (ShakeInfo.DebugDataIndex != INDEX_NONE)
	{
		// Just make the debug info as inactive. It will get culled once it has been displayed
		// long enough after that.
		ensure(DebugShakes.IsValidIndex(ShakeInfo.DebugDataIndex));
		ensure(DebugShakes[ShakeInfo.DebugDataIndex].bIsInactive == false);
		DebugShakes[ShakeInfo.DebugDataIndex].bIsInactive = true;
		DebugShakes[ShakeInfo.DebugDataIndex].TimeInactive = 0.f;
	}
}

void UCameraModifier_CameraShake::DisplayDebugGraphs(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay)
{
	const bool bPreviousRecordDebugData = bRecordDebugData;

	// See if we should be recording and display shake data.
	const bool bShowLocationDebug = GShowCameraShakeDebugCVar.GetValueOnGameThread() && 
		GShowCameraShakeDebugLocationCVar.GetValueOnGameThread();
	const bool bShowRotationDebug = GShowCameraShakeDebugCVar.GetValueOnGameThread() && 
		GShowCameraShakeDebugRotationCVar.GetValueOnGameThread();
	bRecordDebugData = bShowLocationDebug || bShowRotationDebug;
	
	if (!bRecordDebugData)
	{
		return;
	}

	if (!bPreviousRecordDebugData)
	{
		// If there are any already running shakes when we start displaying debug info, let's add them
		// to the debug list.
		for (FActiveCameraShakeInfo& ShakeInfo : ActiveShakes)
		{
			if (ShakeInfo.DebugDataIndex == INDEX_NONE)
			{
				AddCameraShakeDebugData(ShakeInfo);
			}
		}
	}

	Canvas->SetDrawColor(FColor::Yellow);
	UFont* DrawFont = GEngine->GetSmallFont();
	const float FontHeight = DrawFont->GetMaxCharHeight();
	
	const bool bDrawLargeGraphs = GCameraShakeDebugLargeGraphCVar.GetValueOnGameThread();

	const float GraphWidth = bDrawLargeGraphs ? 320.f : 160.f;
	const float GraphHeight = bDrawLargeGraphs ? 120.f : 60.0f;
	const float GraphRightXPos = Canvas->SizeX - 20;
	const float GraphLeftXPos = GraphRightXPos - GraphWidth;

	const float DebugDataLimit = GCameraShakeDebugInfoRecordLimitCVar.GetValueOnGameThread();
	const float PixelsPerSecond = GraphWidth / (DebugDataLimit > 0 ? DebugDataLimit : 3.f);

	for (int i = 0; i < DebugShakes.Num(); i++)
	{
		const FCameraShakeDebugData& DebugShake = DebugShakes[i];
		if (DebugShake.DataPoints.IsEmpty())
		{
			continue;
		}

		const float GraphTopYPos = 20 + (i * (GraphHeight + FontHeight + 20));

		// Draw background.
		FLinearColor BackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.7f);
		FCanvasTileItem BackgroundTile(
				FVector2D(GraphLeftXPos, GraphTopYPos),
				FVector2D(GraphWidth, GraphHeight),
				BackgroundColor);
		BackgroundTile.BlendMode = SE_BLEND_AlphaBlend;
		Canvas->DrawItem(BackgroundTile);

		// Draw shake name.
		Canvas->DrawText(DrawFont, *DebugShake.ShakeInstanceName, GraphLeftXPos, GraphTopYPos + GraphHeight + 10);

		FBatchedElements* BatchedElements = Canvas->Canvas->GetBatchedElements(FCanvas::ET_Line);
		FHitProxyId HitProxyId = Canvas->Canvas->GetHitProxyId();
		const float GraphBottomYPos = GraphTopYPos + GraphHeight;

		// Compute vertical scale based on min/max values.
		float MaxLocationValue = 0.f;
		float MinLocationValue = 0.f;
		float MaxRotationValue = 0.f;
		float MinRotationValue = 0.f;

		for (int32 DataIndex = 0; DataIndex < DebugShake.DataPoints.Num(); ++DataIndex)
		{
			const FCameraShakeDebugDataPoint& DataPoint = DebugShake.DataPoints[DataIndex];

			MaxLocationValue = FMath::Max(MaxLocationValue, DataPoint.DeltaLocation.X);
			MaxLocationValue = FMath::Max(MaxLocationValue, DataPoint.DeltaLocation.Y);
			MaxLocationValue = FMath::Max(MaxLocationValue, DataPoint.DeltaLocation.Z);
			MinLocationValue = FMath::Min(MinLocationValue, DataPoint.DeltaLocation.X);
			MinLocationValue = FMath::Min(MinLocationValue, DataPoint.DeltaLocation.Y);
			MinLocationValue = FMath::Min(MinLocationValue, DataPoint.DeltaLocation.Z);

			MaxRotationValue = FMath::Max(MaxRotationValue, DataPoint.DeltaRotation.Yaw);
			MaxRotationValue = FMath::Max(MaxRotationValue, DataPoint.DeltaRotation.Pitch);
			MaxRotationValue = FMath::Max(MaxRotationValue, DataPoint.DeltaRotation.Roll);
			MinRotationValue = FMath::Min(MinRotationValue, DataPoint.DeltaRotation.Yaw);
			MinRotationValue = FMath::Min(MinRotationValue, DataPoint.DeltaRotation.Pitch);
			MinRotationValue = FMath::Min(MinRotationValue, DataPoint.DeltaRotation.Roll);
		}

		const float Threshold = 0.005f;
		if (MaxLocationValue - MinLocationValue < Threshold * 2.f)
		{
			MaxLocationValue += Threshold;
			MinLocationValue -= Threshold;
		}
		if (MaxRotationValue - MinRotationValue < Threshold * 2.f)
		{
			MaxRotationValue += Threshold;
			MinRotationValue -= Threshold;
		}

		// Draw min/max.
		if (bShowLocationDebug && bShowRotationDebug)
		{
			int MaxLabelHeight, MaxLabelWidth;
			FString MaxLabel = FString::Printf(TEXT("L=%.02f R=%.02f"), MaxLocationValue, MaxRotationValue);
			DrawFont->GetStringHeightAndWidth(MaxLabel, MaxLabelHeight, MaxLabelWidth);

			int MinLabelHeight, MinLabelWidth;
			FString MinLabel = FString::Printf(TEXT("L=%.02f R=%.02f"), MinLocationValue, MinRotationValue);
			DrawFont->GetStringHeightAndWidth(MinLabel, MinLabelHeight, MinLabelWidth);

			Canvas->DrawText(DrawFont, MaxLabel, GraphLeftXPos - MaxLabelWidth - 10, GraphTopYPos);
			Canvas->DrawText(DrawFont, MinLabel, GraphLeftXPos - MinLabelWidth - 10, GraphTopYPos + GraphHeight - MinLabelHeight);
		}
		else if (bShowLocationDebug)
		{
			int MaxLabelHeight, MaxLabelWidth;
			FString MaxLabel = FString::Printf(TEXT("%.02f"), MaxLocationValue);
			DrawFont->GetStringHeightAndWidth(MaxLabel, MaxLabelHeight, MaxLabelWidth);

			int MinLabelHeight, MinLabelWidth;
			FString MinLabel = FString::Printf(TEXT("%.02f"), MinLocationValue);
			DrawFont->GetStringHeightAndWidth(MinLabel, MinLabelHeight, MinLabelWidth);

			Canvas->DrawText(DrawFont, MaxLabel, GraphLeftXPos - MaxLabelWidth - 10, GraphTopYPos);
			Canvas->DrawText(DrawFont, MinLabel, GraphLeftXPos - MinLabelWidth - 10, GraphTopYPos + GraphHeight - MinLabelHeight);
		}
		else if (bShowRotationDebug)
		{
			int MaxLabelHeight, MaxLabelWidth;
			FString MaxLabel = FString::Printf(TEXT("%.02f"), MaxRotationValue);
			DrawFont->GetStringHeightAndWidth(MaxLabel, MaxLabelHeight, MaxLabelWidth);

			int MinLabelHeight, MinLabelWidth;
			FString MinLabel = FString::Printf(TEXT("%.02f"), MinRotationValue);
			DrawFont->GetStringHeightAndWidth(MinLabel, MinLabelHeight, MinLabelWidth);

			Canvas->DrawText(DrawFont, MaxLabel, GraphLeftXPos - MaxLabelWidth - 10, GraphTopYPos);
			Canvas->DrawText(DrawFont, MinLabel, GraphLeftXPos - MinLabelWidth - 10, GraphTopYPos + GraphHeight - MinLabelHeight);
		}

		// Draw curves.
		const float LocationUnitsPerPixel = (MaxLocationValue - MinLocationValue) / GraphHeight;
		const float RotationUnitsPerPixel = (MaxRotationValue - MinRotationValue) / GraphHeight;

		FLinearColor LocationXStatColor(1.0f, 0.1f, 0.1f);		// Red
		FLinearColor LocationYStatColor(0.1f, 1.0f, 0.1f);		// Green
		FLinearColor LocationZStatColor(0.1f, 0.1f, 1.0f);		// Blue

		FLinearColor RotationXStatColor(1.0f, 1.0f, 0.1f);		// Yellow
		FLinearColor RotationYStatColor(1.0f, 0.5f, 0.1f);		// Orange
		FLinearColor RotationZStatColor(0.1f, 0.8f, 0.8f);		// Cyan

		const float TimeOffset = DebugShake.DataPoints[0].AccumulatedTime;

		for (int32 DataIndex = 1; DataIndex < DebugShake.DataPoints.Num(); ++DataIndex)
		{
			const FCameraShakeDebugDataPoint& PrevDataPoint = DebugShake.DataPoints[DataIndex - 1];
			const float PrevOffsetX = (PrevDataPoint.AccumulatedTime - TimeOffset) * PixelsPerSecond;

			const FCameraShakeDebugDataPoint& NextDataPoint = DebugShake.DataPoints[DataIndex];
			const float NextOffsetX = (NextDataPoint.AccumulatedTime - TimeOffset) * PixelsPerSecond;

			if (bShowLocationDebug)
			{
				const FVector PrevLocationPixels(
					(PrevDataPoint.DeltaLocation.X - MinLocationValue) / LocationUnitsPerPixel,
					(PrevDataPoint.DeltaLocation.Y - MinLocationValue) / LocationUnitsPerPixel,
					(PrevDataPoint.DeltaLocation.Z - MinLocationValue) / LocationUnitsPerPixel);

				const FVector NextLocationPixels(
					(NextDataPoint.DeltaLocation.X - MinLocationValue) / LocationUnitsPerPixel,
					(NextDataPoint.DeltaLocation.Y - MinLocationValue) / LocationUnitsPerPixel,
					(NextDataPoint.DeltaLocation.Z - MinLocationValue) / LocationUnitsPerPixel);

				const FVector LineStartLocationX(GraphLeftXPos + PrevOffsetX, GraphBottomYPos - PrevLocationPixels.X, 0.0f);
				const FVector LineStartLocationY(GraphLeftXPos + PrevOffsetX, GraphBottomYPos - PrevLocationPixels.Y, 0.0f);
				const FVector LineStartLocationZ(GraphLeftXPos + PrevOffsetX, GraphBottomYPos - PrevLocationPixels.Z, 0.0f);

				const FVector LineEndLocationX(GraphLeftXPos + NextOffsetX, GraphBottomYPos - NextLocationPixels.X, 0.0f);
				const FVector LineEndLocationY(GraphLeftXPos + NextOffsetX, GraphBottomYPos - NextLocationPixels.Y, 0.0f);
				const FVector LineEndLocationZ(GraphLeftXPos + NextOffsetX, GraphBottomYPos - NextLocationPixels.Z, 0.0f);

				BatchedElements->AddLine(LineStartLocationX, LineEndLocationX, LocationXStatColor, HitProxyId);
				BatchedElements->AddLine(LineStartLocationY, LineEndLocationY, LocationYStatColor, HitProxyId);
				BatchedElements->AddLine(LineStartLocationZ, LineEndLocationZ, LocationZStatColor, HitProxyId);
			}

			if (bShowRotationDebug)
			{
				const FVector PrevRotationPixels( 
					(PrevDataPoint.DeltaRotation.Yaw - MinRotationValue) / RotationUnitsPerPixel, 
					(PrevDataPoint.DeltaRotation.Pitch - MinRotationValue) / RotationUnitsPerPixel,
					(PrevDataPoint.DeltaRotation.Roll - MinRotationValue) / RotationUnitsPerPixel);
			
				const FVector NextRotationPixels( 
					(NextDataPoint.DeltaRotation.Yaw - MinRotationValue) / RotationUnitsPerPixel,
					(NextDataPoint.DeltaRotation.Pitch - MinRotationValue) / RotationUnitsPerPixel,
					(NextDataPoint.DeltaRotation.Roll - MinRotationValue) / RotationUnitsPerPixel);

				const FVector LineStartRotationX(GraphLeftXPos + PrevOffsetX, GraphBottomYPos - PrevRotationPixels.X, 0.0f);
				const FVector LineStartRotationY(GraphLeftXPos + PrevOffsetX, GraphBottomYPos - PrevRotationPixels.Y, 0.0f);
				const FVector LineStartRotationZ(GraphLeftXPos + PrevOffsetX, GraphBottomYPos - PrevRotationPixels.Z, 0.0f);

				const FVector LineEndRotationX(GraphLeftXPos + NextOffsetX, GraphBottomYPos - NextRotationPixels.X, 0.0f);
				const FVector LineEndRotationY(GraphLeftXPos + NextOffsetX, GraphBottomYPos - NextRotationPixels.Y, 0.0f);
				const FVector LineEndRotationZ(GraphLeftXPos + NextOffsetX, GraphBottomYPos - NextRotationPixels.Z, 0.0f);

				BatchedElements->AddLine(LineStartRotationX, LineEndRotationX, RotationXStatColor, HitProxyId);
				BatchedElements->AddLine(LineStartRotationY, LineEndRotationY, RotationYStatColor, HitProxyId);
				BatchedElements->AddLine(LineStartRotationZ, LineEndRotationZ, RotationZStatColor, HitProxyId);
			}
		}
	}
}

#endif


