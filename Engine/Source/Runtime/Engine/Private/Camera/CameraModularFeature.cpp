// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraModularFeature.h"
#include "UObject/NameTypes.h"

FName ICameraModularFeature::GetModularFeatureName()
{
	static FName FeatureName = FName("Camera");
	return FeatureName;
}
