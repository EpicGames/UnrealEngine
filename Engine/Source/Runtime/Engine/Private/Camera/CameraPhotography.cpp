// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraPhotography.h"

#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "CameraPhotographyModule.h"
#include "Engine/World.h"

DEFINE_LOG_CATEGORY_STATIC(LogCameraPhotography, Log, All);

/////////////////////////////////////////////////

static TAutoConsoleVariable<int32> CVarPhotographyAvailable(
	TEXT("r.Photography.Available"),
	1,
	TEXT("(Read-only) If 1, the photography system is potentially available to the user.\n")
	TEXT("Otherwise, a functioning back-end is not available."), 
	ECVF_ReadOnly);

/////////////////////////////////////////////////
// FCameraPhotography internals

FCameraPhotographyManager::FCameraPhotographyManager()
	: ActiveImpl(nullptr)
{
	bool bIsSupported = false;

	// initialize any externally-implemented photography implementations (we delay load initialize the array so any plugins have had time to load)
	TArray<ICameraPhotographyModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<ICameraPhotographyModule>(ICameraPhotographyModule::GetModularFeatureName());

	//we take the first one since we don't have a runtime prioritization scheme for multiple photography implementations.
	for (auto CameraPhotoIt = PluginImplementations.CreateIterator(); CameraPhotoIt && !ActiveImpl.IsValid(); ++CameraPhotoIt)
	{
		ActiveImpl = (*CameraPhotoIt)->CreateCameraPhotography();
	}

	if (ActiveImpl.IsValid())
	{
		UE_LOGF(LogCameraPhotography, Log, "Photography camera created.  Provider=%ls, Supported=%d", ActiveImpl->GetProviderName(), ActiveImpl->IsSupported());
		bIsSupported = ActiveImpl->IsSupported();
	}

	CVarPhotographyAvailable->Set(bIsSupported ? 1 : 0);	
}

FCameraPhotographyManager::~FCameraPhotographyManager()
{
	if (ActiveImpl.IsValid())
	{
		UE_LOGF(LogCameraPhotography, Log, "Photography camera destroyed.  Provider=%ls, Supported=%d", ActiveImpl->GetProviderName(), ActiveImpl->IsSupported());		
		ActiveImpl.Reset();
	}	
}


/////////////////////////////////////////////////
// FCameraPhotography Public API

FCameraPhotographyManager* FCameraPhotographyManager::Singleton = nullptr;

bool FCameraPhotographyManager::IsSupported(UWorld* InWorld)
{
	//we don't want this running on dedicated servers
	if(InWorld && InWorld->GetNetMode() != NM_DedicatedServer)
	{
		if (ICameraPhotography* Impl = Get().ActiveImpl.Get())
		{
			return Impl->IsSupported();
		}
	}
	return false;	
}


FCameraPhotographyManager& FCameraPhotographyManager::Get()
{
	if (nullptr == Singleton)
	{
		Singleton = new FCameraPhotographyManager();
		FCoreDelegates::OnExit.AddStatic(Destroy);
	}

	return *Singleton;
}

void FCameraPhotographyManager::Destroy()
{
	delete Singleton;
	Singleton = nullptr;
}

bool FCameraPhotographyManager::UpdateCamera(FMinimalViewInfo& InOutPOV, APlayerCameraManager* PCMgr)
{
	if (ActiveImpl.IsValid())
	{
		return ActiveImpl->UpdateCamera(InOutPOV, PCMgr);
	}
	return false;
}

void FCameraPhotographyManager::UpdatePostProcessing(FPostProcessSettings& InOutPostProcessingSettings)
{
	if (ActiveImpl.IsValid())
	{
		ActiveImpl->UpdatePostProcessing(InOutPostProcessingSettings);
	}
}

void FCameraPhotographyManager::StartSession()
{
	if (ActiveImpl.IsValid())
	{
		ActiveImpl->StartSession();
	}
}

void FCameraPhotographyManager::StopSession()
{
	if (ActiveImpl.IsValid())
	{
		ActiveImpl->StopSession();
	}
}

void FCameraPhotographyManager::SetUIControlVisibility(uint8 UIControlTarget, bool bIsVisible)
{
	if (ActiveImpl.IsValid())
	{
		ActiveImpl->SetUIControlVisibility(UIControlTarget, bIsVisible);
	}
}

void FCameraPhotographyManager::DefaultConstrainCamera(const FVector NewCameraLocation, const FVector PreviousCameraLocation, const FVector OriginalCameraLocation, FVector& OutCameraLocation, APlayerCameraManager* PCMgr)
{
	// let proposed camera through unmodified by default
	OutCameraLocation = NewCameraLocation;

	if (ActiveImpl.IsValid())
	{
		ActiveImpl->DefaultConstrainCamera(NewCameraLocation, PreviousCameraLocation, OriginalCameraLocation, OutCameraLocation, PCMgr);
	}
}



