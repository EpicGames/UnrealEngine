// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraShakeBase.h"
#include "Camera/CameraAnimationHelper.h"
#include "Camera/PlayerCameraManager.h"
#include "Engine/Engine.h"
#include "IXRTrackingSystem.h"
#include "Math/RotationMatrix.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "Stats/Stats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeBase)

DECLARE_CYCLE_STAT(TEXT("CameraShakeStartShake"), STAT_StartShake, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("CameraShakeUpdateShake"), STAT_UpdateShake, STATGROUP_Game);

TAutoConsoleVariable<bool> GCameraShakeLegacyPostProcessBlending(
	TEXT("r.CameraShake.LegacyPostProcessBlending"),
	false,
	TEXT("Blend camera shake post process settings under the main camera instead of over it"));

UCameraShakeBase::UCameraShakeBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bSingleInstance(false)
	, ShakeScale(1.f)
	, PlaySpace(ECameraShakePlaySpace::CameraLocal)
	, bIsActive(false)
{
	RootShakePattern = static_cast<UCameraShakePattern*>(ObjectInitializer.CreateDefaultSubobject(
			this, 
			TEXT("RootShakePattern"), 
			UCameraShakePattern::StaticClass(),
			nullptr,	// No class to create by default
			false,		// Not required
			false		// Not transient
			));
}

FCameraShakeDuration UCameraShakeBase::GetCameraShakeDuration() const
{
	FCameraShakeInfo TempInfo;
	GetShakeInfo(TempInfo);
	return TempInfo.Duration;
}

FCameraShakeState::FCameraShakeState()
	: ElapsedTime(0.f)
	, CurrentBlendInTime(0.f)
	, CurrentBlendOutTime(0.f)
	, bIsBlendingIn(false)
	, bIsBlendingOut(false)
	, bIsPlaying(false)
	, bHasBlendIn(false)
	, bHasBlendOut(false)
{
}

void FCameraShakeState::Start(const FCameraShakeInfo& InShakeInfo)
{
	Start(InShakeInfo, TOptional<float>());
}

void FCameraShakeState::Start(const FCameraShakeInfo& InShakeInfo, TOptional<float> InDurationOverride)
{
	const FCameraShakeState PrevState(*this);

	// Cache a few things about the shake.
	ShakeInfo = InShakeInfo;
	bHasBlendIn = ShakeInfo.BlendIn > 0.f;
	bHasBlendOut = ShakeInfo.BlendOut > 0.f;

	if (InDurationOverride.IsSet())
	{
		ShakeInfo.Duration = FCameraShakeDuration(InDurationOverride.GetValue());
	}

	// Initialize our running state.
	InitializePlaying();

	if (PrevState.bIsPlaying)
	{
		// Single instance shake is being restarted... let's see if we need to
		// reverse a blend-out into a blend-in.
		if (bHasBlendIn && PrevState.bHasBlendOut && PrevState.bIsBlendingOut)
		{
			// We had started blending out... let's start at an equivalent weight into the blend in.
			CurrentBlendInTime = ShakeInfo.BlendIn * (1.f - PrevState.CurrentBlendOutTime / PrevState.ShakeInfo.BlendOut);
		}
		else if (bHasBlendIn && !PrevState.bHasBlendOut)
		{
			// We have no blend out, so we were still at 100%. But we don't want to suddenly drop 
			// to 0% at the beginning of the blend in, so we skip the blend-in.
			bIsBlendingIn = false;
		}
	}
}

void FCameraShakeState::Start(const UCameraShakePattern* InShakePattern)
{
	check(InShakePattern);
	FCameraShakeInfo Info;
	InShakePattern->GetShakePatternInfo(Info);
	Start(Info);
}

void FCameraShakeState::Start(const UCameraShakePattern* InShakePattern, const FCameraShakePatternStartParams& InParams)
{
	check(InShakePattern);
	FCameraShakeInfo Info;
	InShakePattern->GetShakePatternInfo(Info);
	TOptional<float> DurationOverride;
	if (InParams.bOverrideDuration)
	{
		DurationOverride = InParams.DurationOverride;
	}
	Start(Info, DurationOverride);
}

void FCameraShakeState::InitializePlaying()
{
	ElapsedTime = 0.f;
	CurrentBlendInTime = 0.f;
	CurrentBlendOutTime = 0.f;
	bIsBlendingIn = bHasBlendIn;
	bIsBlendingOut = false;
	bIsPlaying = true;
}

float FCameraShakeState::Update(float DeltaTime)
{
	// If we have duration information for our shake, we can do all the time-keeping stuff ourselves.
	// This includes figuring out if the shake is finished, and what kind of blend in/out weight
	// we should apply.
	float BlendingWeight = 1.f;

	// Advance shake and blending times.
	ElapsedTime += DeltaTime;
	if (bIsBlendingIn)
	{
		CurrentBlendInTime += DeltaTime;
	}
	if (bIsBlendingOut)
	{
		CurrentBlendOutTime += DeltaTime;
	}

	// Advance progress into the shake.
	if (HasDuration())
	{
		const float ShakeDuration = ShakeInfo.Duration.Get();
		if (ElapsedTime < 0.f || ElapsedTime >= ShakeDuration)
		{
			// The shake has ended, or hasn't started yet (which can happen if we update backwards)
			bIsPlaying = false;
			return 0.f;
		}

		const float DurationRemaining = (ShakeDuration - ElapsedTime);
		if (bHasBlendOut && !bIsBlendingOut && DurationRemaining < ShakeInfo.BlendOut)
		{
			// We started blending out.
			bIsBlendingOut = true;
			CurrentBlendOutTime = (ShakeInfo.BlendOut - DurationRemaining);
		}
	}

	// Compute blend-in and blend-out weight.
	if (bHasBlendIn && bIsBlendingIn)
	{
		if (CurrentBlendInTime < ShakeInfo.BlendIn)
		{
			BlendingWeight *= (CurrentBlendInTime / ShakeInfo.BlendIn);
		}
		else
		{
			// Finished blending in!
			bIsBlendingIn = false;
			CurrentBlendInTime = ShakeInfo.BlendIn;
		}
	}
	if (bHasBlendOut && bIsBlendingOut)
	{
		if (CurrentBlendOutTime < ShakeInfo.BlendOut)
		{
			BlendingWeight *= (1.f - CurrentBlendOutTime / ShakeInfo.BlendOut);
		}
		else
		{
			// Finished blending out!
			bIsBlendingOut = false;
			CurrentBlendOutTime = ShakeInfo.BlendOut;
			// We also end the shake itself. In most cases we would have hit the similar case
			// above already, but if we have an infinite shake we have no duration to reach the end
			// of so we only finish here.
			bIsPlaying = false;
			return 0.f;
		}
	}
	return BlendingWeight;
}

float FCameraShakeState::Scrub(float AbsoluteTime)
{
	// Reset the state to active, at the beginning, and update from there.
	InitializePlaying();
	return Update(AbsoluteTime);
}

void FCameraShakeState::Stop(bool bImmediately)
{
	// For stopping immediately, we don't do anything besides render the shake inactive.
	if (bImmediately || !bHasBlendOut)
	{
		bIsPlaying = false;
	}
	// For stopping with a "graceful" blend-out:
	// - If we are already blending out, let's keep doing that and not change anything.
	// - If we are not, let's start blending out.
	else if (bHasBlendOut && !bIsBlendingOut)
	{
		bIsBlendingOut = true;
		CurrentBlendOutTime = 0.f;
	}
}

FCameraShakePatternUpdateParams FCameraShakePatternScrubParams::ToUpdateParams() const
{
	FCameraShakePatternUpdateParams UpdateParams(POV);
	UpdateParams.DeltaTime = AbsoluteTime;
	UpdateParams.ShakeScale = ShakeScale;
	UpdateParams.DynamicScale = DynamicScale;
	return UpdateParams;
}

void FCameraShakePatternUpdateResult::ApplyScale(float InScale)
{
	if (ensureMsgf(!EnumHasAnyFlags(Flags, ECameraShakePatternUpdateResultFlags::ApplyAsAbsolute),
			TEXT("Can't scale absolute shake result")))
	{
		Location *= InScale;
		Rotation *= InScale;
		FOV *= InScale;
		PostProcessBlendWeight *= InScale;
	}
}

void UCameraShakeBase::GetCameraShakeBlendTimes(float& OutBlendIn, float& OutBlendOut) const
{
	FCameraShakeInfo TempInfo;
	GetShakeInfo(TempInfo);
	OutBlendIn = TempInfo.BlendIn;
	OutBlendOut = TempInfo.BlendOut;
}

void UCameraShakeBase::SetRootShakePattern(UCameraShakePattern* InPattern)
{
	if (ensureMsgf(!bIsActive, TEXT("Can't change the root shake pattern while the shake is running!")))
	{
		RootShakePattern = InPattern;
	}
}

void UCameraShakeBase::GetShakeInfo(FCameraShakeInfo& OutInfo) const
{
	if (RootShakePattern)
	{
		RootShakePattern->GetShakePatternInfo(OutInfo);
	}
}

void UCameraShakeBase::StartShake(APlayerCameraManager* Camera, float Scale, ECameraShakePlaySpace InPlaySpace, FRotator UserPlaySpaceRot)
{
	FCameraShakeBaseStartParams Params;
	Params.CameraManager = Camera;
	Params.Scale = Scale;
	Params.PlaySpace = InPlaySpace;
	Params.UserPlaySpaceRot = UserPlaySpaceRot;
	StartShake(Params);
}

void UCameraShakeBase::StartShake(const FCameraShakeBaseStartParams& Params)
{
	SCOPE_CYCLE_COUNTER(STAT_StartShake);

	// Check that we were correctly stopped before we are asked to play again.
	// Note that single-instance shakes can be restarted while they're running.
	if (!ensureMsgf(!bIsActive || bSingleInstance, TEXT("Starting to play a shake that was already playing.")))
	{
		return;
	}

	// Remember the various settings for this run.
	// Note that the camera manager can be null, for example in unit tests.
	CameraManager = Params.CameraManager;
	ShakeScale = Params.Scale;
	PlaySpace = Params.PlaySpace;
	UserPlaySpaceMatrix = (Params.PlaySpace == ECameraShakePlaySpace::UserDefined) ? 
		FRotationMatrix(Params.UserPlaySpaceRot) : FRotationMatrix::Identity;

	const bool bIsRestarting = bIsActive;
	bIsActive = true;

	// Let the root pattern initialize itself.
	if (RootShakePattern)
	{
		FCameraShakePatternStartParams StartParams;
		StartParams.bIsRestarting = bIsRestarting;
		StartParams.bOverrideDuration = Params.DurationOverride.IsSet();
		StartParams.DurationOverride = Params.DurationOverride.Get(0.f);
		RootShakePattern->StartShakePattern(StartParams);
	}
}

void UCameraShakeBase::UpdateAndApplyCameraShake(float DeltaTime, float Alpha, FMinimalViewInfo& InOutPOV)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateShake);
	SCOPE_CYCLE_UOBJECT(This, this);

	if (!ensureMsgf(bIsActive, TEXT("Updating a camera shake that wasn't started with a call to StartShake!")))
	{
		return;
	}

	if (RootShakePattern)
	{
		// Make the sub-class do the actual work.
		FCameraShakePatternUpdateParams Params(InOutPOV);
		Params.DeltaTime = DeltaTime;
		Params.ShakeScale = ShakeScale;
		Params.DynamicScale = Alpha;

		// Result object is initialized with zero values since the default flags make us handle it
		// as an additive offset.
		FCameraShakePatternUpdateResult Result;

		RootShakePattern->UpdateShakePattern(Params, Result);

		if (!RootShakePattern->IsFinished())
		{
			// Apply the result to the given view info.
			FCameraShakeApplyResultParams ApplyParams;
			ApplyParams.Scale = Params.GetTotalScale();
			ApplyParams.PlaySpace = PlaySpace;
			ApplyParams.UserPlaySpaceMatrix = UserPlaySpaceMatrix;
			ApplyParams.CameraManager = CameraManager;
			ApplyResult(ApplyParams, Result, InOutPOV);
		}
	}
}

void UCameraShakeBase::ScrubAndApplyCameraShake(float AbsoluteTime, float Alpha, FMinimalViewInfo& InOutPOV)
{
	// This code is similar to the above UpdateAndApplyCameraShake method, but calls the scrub method
	// on the state manager and root pattern instead of the update method.

	SCOPE_CYCLE_COUNTER(STAT_UpdateShake);
	SCOPE_CYCLE_UOBJECT(This, this);

	if (!ensureMsgf(bIsActive, TEXT("Updating a camera shake that wasn't started with a call to StartShake!")))
	{
		return;
	}

	if (RootShakePattern)
	{
		// Make the sub-class do the actual work.
		FCameraShakePatternScrubParams Params(InOutPOV);
		Params.AbsoluteTime = AbsoluteTime;
		Params.ShakeScale = ShakeScale;
		Params.DynamicScale = Alpha;

		// Result object is initialized with zero values since the default flags make us handle it
		// as an additive offset.
		FCameraShakePatternUpdateResult Result;

		RootShakePattern->ScrubShakePattern(Params, Result);

		if (!RootShakePattern->IsFinished())
		{
			// Apply the result to the given view info.
			FCameraShakeApplyResultParams ApplyParams;
			ApplyParams.Scale = Params.GetTotalScale();
			ApplyParams.PlaySpace = PlaySpace;
			ApplyParams.UserPlaySpaceMatrix = UserPlaySpaceMatrix;
			ApplyParams.CameraManager = CameraManager;
			ApplyResult(ApplyParams, Result, InOutPOV);
		}
	}
}

bool UCameraShakeBase::IsFinished() const
{
	if (RootShakePattern)
	{
		// Ask the root pattern whether it's finished.
		return RootShakePattern->IsFinished();
	}
	else
	{
		// We have no root pattern, we don't have anything to do.
		return true;
	}
}

void UCameraShakeBase::StopShake(bool bImmediately)
{
	if (!ensureMsgf(bIsActive, TEXT("Stopping a shake that wasn't active")))
	{
		return;
	}

	// Let the root pattern do any custom logic.
	if (RootShakePattern)
	{
		FCameraShakePatternStopParams StopParams;
		StopParams.bImmediately = bImmediately;
		RootShakePattern->StopShakePattern(StopParams);
	}
}

void UCameraShakeBase::TeardownShake()
{
	if (RootShakePattern)
	{
		RootShakePattern->TeardownShakePattern();
	}

	bIsActive = false;
}

void UCameraShakeBase::ApplyResult(const FCameraShakeApplyResultParams& ApplyParams, const FCameraShakePatternUpdateResult& InResult, FMinimalViewInfo& InOutPOV)
{
	FCameraShakePatternUpdateResult TempResult(InResult);

	// If the sub-class gave us a delta-transform, we can help with some of the basic functionality
	// of a camera shake... namely: apply shake scaling, system limits, and play space transformation.
	if (!EnumHasAnyFlags(TempResult.Flags, ECameraShakePatternUpdateResultFlags::ApplyAsAbsolute))
	{
		if (!EnumHasAnyFlags(TempResult.Flags, ECameraShakePatternUpdateResultFlags::SkipAutoScale))
		{
			ApplyScale(ApplyParams.Scale, TempResult);
		}

		ApplyLimits(InOutPOV, TempResult);

		if (!EnumHasAnyFlags(TempResult.Flags, ECameraShakePatternUpdateResultFlags::SkipAutoPlaySpace))
		{
			ApplyPlaySpace(ApplyParams.PlaySpace, ApplyParams.UserPlaySpaceMatrix, InOutPOV, TempResult);
		}
	}

	// Now we can apply the shake to the camera matrix.
	if (EnumHasAnyFlags(TempResult.Flags, ECameraShakePatternUpdateResultFlags::ApplyAsAbsolute))
	{
		InOutPOV.Location = TempResult.Location;
		InOutPOV.Rotation = TempResult.Rotation;
		InOutPOV.FOV = TempResult.FOV;
	}
	else
	{
		InOutPOV.Location += TempResult.Location;
		InOutPOV.Rotation += TempResult.Rotation;
		InOutPOV.FOV += TempResult.FOV;
	}

	if (TempResult.PostProcessBlendWeight > 0.f)
	{
		// If we have a camera manager, post-process settings go there. Otherwise, let's put them on
		// the view-info.
		if (ApplyParams.CameraManager.IsValid())
		{
			EViewTargetBlendOrder CameraShakeBlendOrder = GCameraShakeLegacyPostProcessBlending.GetValueOnGameThread() ? VTBlendOrder_Base : VTBlendOrder_Override;
			ApplyParams.CameraManager->AddCachedPPBlend(TempResult.PostProcessSettings, TempResult.PostProcessBlendWeight, CameraShakeBlendOrder);
		}
		else
		{
			InOutPOV.PostProcessSettings = TempResult.PostProcessSettings;
			InOutPOV.PostProcessBlendWeight = TempResult.PostProcessBlendWeight;
		}
	}
}

void UCameraShakeBase::ApplyScale(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& InOutResult)
{
	InOutResult.ApplyScale(Params.GetTotalScale());
}

void UCameraShakeBase::ApplyScale(float Scale, FCameraShakePatternUpdateResult& InOutResult)
{
	InOutResult.ApplyScale(Scale);
}

void UCameraShakeBase::ApplyLimits(const FMinimalViewInfo& InPOV, FCameraShakePatternUpdateResult& InOutResult)
{
	// Don't allow shake to flip pitch past vertical, if not using a headset.
	// If using a headset, we can't limit the camera locked to your head.
	if (!GEngine->XRSystem.IsValid() || !GEngine->XRSystem->IsHeadTrackingAllowed())
	{
		// Find normalized result when combined, and remove any offset that would push it past the limit.
		const float NormalizedInputPitch = FRotator::NormalizeAxis(InPOV.Rotation.Pitch);
		const float NormalizedOutputPitchOffset = FRotator::NormalizeAxis(InOutResult.Rotation.Pitch);
		InOutResult.Rotation.Pitch = FMath::ClampAngle(NormalizedInputPitch + NormalizedOutputPitchOffset, -89.9f, 89.9f) - NormalizedInputPitch;
	}
}

void UCameraShakeBase::ApplyPlaySpace(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& InOutResult) const
{
	ApplyPlaySpace(PlaySpace, UserPlaySpaceMatrix, Params.POV, InOutResult);
}

void UCameraShakeBase::ApplyPlaySpace(ECameraShakePlaySpace PlaySpace, FMatrix UserPlaySpaceMatrix, const FMinimalViewInfo& InPOV, FCameraShakePatternUpdateResult& InOutResult)
{
	// Orient the shake according to the play space.
	const bool bIsCameraLocal = (PlaySpace == ECameraShakePlaySpace::CameraLocal);
	const FCameraAnimationHelperOffset CameraOffset { InOutResult.Location, InOutResult.Rotation };
	if (bIsCameraLocal)
	{
		FCameraAnimationHelper::ApplyOffset(InPOV, CameraOffset, InOutResult.Location, InOutResult.Rotation);
	}
	else
	{
		FCameraAnimationHelper::ApplyOffset(UserPlaySpaceMatrix, InPOV, CameraOffset, InOutResult.Location, InOutResult.Rotation);
	}

	// We have a final location/rotation for the camera, so it should be applied verbatim.
	InOutResult.Flags = (InOutResult.Flags | ECameraShakePatternUpdateResultFlags::ApplyAsAbsolute);

	// And since we set that flag, we need to make the FOV absolute too.
	InOutResult.FOV = InPOV.FOV + InOutResult.FOV;
}

UCameraShakePattern::UCameraShakePattern(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UCameraShakeBase* UCameraShakePattern::GetShakeInstance() const
{
	return GetTypedOuter<UCameraShakeBase>();
}

void UCameraShakePattern::GetShakePatternInfo(FCameraShakeInfo& OutInfo) const
{
	GetShakePatternInfoImpl(OutInfo);
}

void UCameraShakePattern::StartShakePattern(const FCameraShakePatternStartParams& Params)
{
	StartShakePatternImpl(Params);
}

void UCameraShakePattern::UpdateShakePattern(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	UpdateShakePatternImpl(Params, OutResult);
}

void UCameraShakePattern::ScrubShakePattern(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	ScrubShakePatternImpl(Params, OutResult);
}

bool UCameraShakePattern::IsFinished() const
{
	return IsFinishedImpl();
}

void UCameraShakePattern::StopShakePattern(const FCameraShakePatternStopParams& Params)
{
	StopShakePatternImpl(Params);
}

void UCameraShakePattern::TeardownShakePattern()
{
	TeardownShakePatternImpl();
}

