// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraShakeSourceActor.h"
#include "Camera/CameraShakeSourceComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeSourceActor)


ACameraShakeSourceActor::ACameraShakeSourceActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CameraShakeSourceComponent = ObjectInitializer.CreateDefaultSubobject<UCameraShakeSourceComponent>(this, TEXT("CameraShakeSourceComponent"));
	RootComponent = CameraShakeSourceComponent;
}

