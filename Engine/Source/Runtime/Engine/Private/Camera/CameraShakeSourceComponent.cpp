// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraShakeSourceComponent.h"
#include "Camera/CameraShakeBase.h"
#include "Camera/CameraModifier_CameraShake.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeSourceComponent)

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "UObject/UnrealType.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogCameraShakeSourceComponent, Log, All);

UCameraShakeSourceComponent::UCameraShakeSourceComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Attenuation(ECameraShakeAttenuation::Quadratic)
	, InnerAttenuationRadius(100.f)
	, OuterAttenuationRadius(1000.f)
	, bAutoStart(false)
{
#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;

	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UTexture2D> StaticTexture(TEXT("/Engine/EditorResources/S_CameraShakeSource"));

		EditorSpriteTexture = StaticTexture.Object;
		EditorSpriteTextureScale = .5f;
	}
#endif
}

void UCameraShakeSourceComponent::OnRegister()
{
	Super::OnRegister();
	UpdateEditorSpriteTexture();
}

void UCameraShakeSourceComponent::UpdateEditorSpriteTexture()
{
#if WITH_EDITORONLY_DATA
	if (SpriteComponent != nullptr)
	{
		SpriteComponent->SetSprite(EditorSpriteTexture);
		SpriteComponent->SetRelativeScale3D(FVector(EditorSpriteTextureScale));
	}
#endif
}

void UCameraShakeSourceComponent::BeginPlay()
{
	Super::BeginPlay();

	if (bAutoStart)
	{
		Start();
	}
}

void UCameraShakeSourceComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	StopAllCameraShakes();

	Super::EndPlay(EndPlayReason);
}

void UCameraShakeSourceComponent::Start()
{
	if (CameraShake.Get() != nullptr)
	{
		StartCameraShake(CameraShake);
	}
	else
	{
		UE_LOGF(LogCameraShakeSourceComponent, Error, "%ls: No camera shake was specified on this source. Me = %ls, Owner = %ls", ANSI_TO_TCHAR(__FUNCTION__), *this->GetFullName(), *GetOwner()->GetFullName());
	}
}

void UCameraShakeSourceComponent::StartCameraShake(TSubclassOf<UCameraShakeBase> InCameraShake, float Scale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRot)
{
	FCameraShakeSourceComponentStartParams Params;
	Params.ShakeClass = InCameraShake;
	Params.Scale = Scale;
	Params.PlaySpace = PlaySpace;
	Params.UserPlaySpaceRot = UserPlaySpaceRot;
	StartCameraShake(Params);
}

void UCameraShakeSourceComponent::StartCameraShake(const FCameraShakeSourceComponentStartParams& Params)
{
	if (UWorld* World = GetWorld())
	{
		FAddCameraShakeParams ModParams;
		ModParams.SourceComponent = this;
		ModParams.Scale = Params.Scale;
		ModParams.PlaySpace = Params.PlaySpace;
		ModParams.UserPlaySpaceRot = Params.UserPlaySpaceRot;
		ModParams.DurationOverride = Params.DurationOverride;

		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController != nullptr && PlayerController->PlayerCameraManager != nullptr)
			{
				PlayerController->PlayerCameraManager->StartCameraShake(Params.ShakeClass, ModParams);
			}
		}
	}
}

void UCameraShakeSourceComponent::StopAllCameraShakesOfType(TSubclassOf<UCameraShakeBase> InCameraShake, bool bImmediately)
{
	if (UWorld* World = GetWorld())
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController != nullptr && PlayerController->PlayerCameraManager != nullptr)
			{
				PlayerController->PlayerCameraManager->StopAllInstancesOfCameraShakeFromSource(InCameraShake, this, bImmediately);
			}
		}
	}
}

void UCameraShakeSourceComponent::StopAllCameraShakes(bool bImmediately)
{
	if (UWorld* World = GetWorld())
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController != nullptr && PlayerController->PlayerCameraManager != nullptr)
			{
				PlayerController->PlayerCameraManager->StopAllCameraShakesFromSource(this, bImmediately);
			}
		}
	}
}

float UCameraShakeSourceComponent::GetAttenuationFactor(const FVector& Location) const
{
	const FTransform& SourceTransform = GetComponentTransform();
	const FVector LocationToSource = SourceTransform.GetTranslation() - Location;

	float AttFactor = 1.0f;
	switch (Attenuation)
	{
		case ECameraShakeAttenuation::Quadratic:
			AttFactor = 2.0f;
			break;
		default:
			break;
	}

	if (InnerAttenuationRadius < OuterAttenuationRadius)
	{
		float DistFactor = (LocationToSource.Size() - InnerAttenuationRadius) / (OuterAttenuationRadius - InnerAttenuationRadius);
		DistFactor = 1.f - FMath::Clamp(DistFactor, 0.f, 1.f);
		return FMath::Pow(DistFactor, AttFactor);
	}
	else if (OuterAttenuationRadius > 0)
	{
		// Just cut the intensity after the end distance.
		return (LocationToSource.SizeSquared() < FMath::Square(OuterAttenuationRadius)) ? 1.f : 0.f;
	}
	return 1.f;
}

#if WITH_EDITOR

void UCameraShakeSourceComponent::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange != nullptr && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UCameraShakeSourceComponent, CameraShake))
    {
        PreviousCameraShake = CameraShake;
    }

    Super::PreEditChange(PropertyAboutToChange);   
}

void UCameraShakeSourceComponent::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
    Super::PostEditChangeProperty(PropertyChangedEvent);

    if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCameraShakeSourceComponent, CameraShake))
    {
        if (CameraShake != nullptr)
        {
            // Single instance shakes can't be localized since they're automatically recycled and merged.
            // If the user is trying to set it, let's notify them and revert to the previous value.
            const UCameraShakeBase* const CameraShakeCDO = GetDefault<UCameraShakeBase>(CameraShake);
            if (CameraShakeCDO != nullptr && CameraShakeCDO->bSingleInstance)
            {
                FText NotificationText = FText::Format(
                        NSLOCTEXT("CameraShakeSourceComponent", "SingleInstanceCameraShakeNotAllowed", "{0} is a single instance shake, and therefore can't be localized to a source actor."),
                        FText::FromName(CameraShake->GetFName()));
                FNotificationInfo NotificationInfo(NotificationText);
                NotificationInfo.ExpireDuration = 5.f;
                FSlateNotificationManager::Get().AddNotification(NotificationInfo);

                CameraShake = PreviousCameraShake;
            }
        }
    }
}

#endif

