// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraStackTypes.h"
#include "Camera/CameraTypes.h"
#include "Engine/EngineTypes.h"
#include "SceneView.h"
#include "Math/OrthoMatrix.h"
#include "UnrealClient.h"
#include "Math/PerspectiveMatrix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraStackTypes)

static TAutoConsoleVariable<bool> CVarUseLegacyMaintainYFOV(
	TEXT("r.UseLegacyMaintainYFOVViewMatrix"),
	false,
	TEXT("Whether to use the old way to compute perspective view matrices when the aspect ratio constraint is vertical"),
	ECVF_Default
);

static TAutoConsoleVariable<bool> CVarOrthoAllowAutoPlanes(
	TEXT("r.Ortho.AutoPlanes"),
	true,
	TEXT("Globally allow Ortho cameras to utilise the automatic Near/Far plane evaluations."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int> CVarOrthoClampToMaxFPBuffer(
	TEXT("r.Ortho.AutoPlanes.ClampToMaxFPBuffer"),
	1,
	TEXT("When auto evaluating clip planes, determines whether 16bit depth scaling should be used.")
	TEXT("16bit scaling is advantageous for any depth downscaling that occurs (e.g. HZB downscaling uses 16 bit textures instead of 32).")
	TEXT("This feature will calculate the maximum depth scale needed based on the Unreal Unit (cm by default) to Pixel ratio.")
	TEXT("It assumes that we don't need 32bit depth range for smaller scenes, because most actors will be within a reasonable visible frustum")
	TEXT("However it does still scale up to a maximum of UE_OLD_WORLD_MAX which is the typical full range of the depth buffer, so larger scenes still work too."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarOrthoScaleIncrementingUnits(
	TEXT("r.Ortho.AutoPlanes.ScaleIncrementingUnits"),
	true,
	TEXT("Select whether to scale the Near/Far plane Min/Max values as we increase in unit to pixel ratio (i.e. as we go from CM to M to KM)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float>  CVarOrthoAutoDepthScale(
	TEXT("r.Ortho.AutoPlanes.DepthScale"),
	-1.0f,
	TEXT("Allows the 16 bit depth scaling to be adjusted from the  default +FP16 Max (66504.0f)")
	TEXT("This is useful if the far plane doesn't need to be as far away, so it will improve depth deltas"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float>  CVarOrthoAutoPlaneShift(
	TEXT("r.Ortho.AutoPlanes.ShiftPlanes"),
	0.0f,
	TEXT("Shifts the whole frustum in the Z direction.")
	TEXT("This can be useful if, for example you need the Near plane closer to the camera, at the reduction of the Far plane value (e.g. a horizontal 2.5D scene)."),
	ECVF_RenderThreadSafe
);

#if !(UE_BUILD_SHIPPING)
static TAutoConsoleVariable<bool> CVarDebugForceAllCamerasToOrtho(
	TEXT("r.Ortho.Debug.ForceAllCamerasToOrtho"),
	false,
	TEXT("Debug Force all cameras in the scene to use Orthographic views"),
	ECVF_RenderThreadSafe
);


static TAutoConsoleVariable<float> CVarDebugForceCameraOrthoWidth(
	TEXT("r.Ortho.Debug.ForceOrthoWidth"),
	DEFAULT_ORTHOWIDTH,
	TEXT("Debug Force Ortho Width when creating a new camera actor"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarDebugForceUseOrthoAutoPlanes(
	TEXT("r.Ortho.Debug.ForceUseAutoPlanes"),
	true,
	TEXT("Debug Force boolean for whether to use the automatic near and far plane evaluation"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarDebugForceCameraOrthoNearPlane(
	TEXT("r.Ortho.Debug.ForceCameraNearPlane"),
	DEFAULT_ORTHONEARPLANE,
	TEXT("Debug Force Ortho Near Plane when creating a new camera actor"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarDebugForceCameraOrthoFarPlane(
	TEXT("r.Ortho.Debug.ForceCameraFarPlane"),
	UE_OLD_WORLD_MAX,
	TEXT("Debug Force Ortho Far Plane when creating a new camera actor"),
	ECVF_RenderThreadSafe
);
#endif

//////////////////////////////////////////////////////////////////////////
// FMinimalViewInfo

bool FMinimalViewInfo::Equals(const FMinimalViewInfo& OtherInfo) const
{
	return 
		(Location == OtherInfo.Location) &&
		(Rotation == OtherInfo.Rotation) &&
		(FOV == OtherInfo.FOV) &&
		(FirstPersonFOV == OtherInfo.FirstPersonFOV) &&
		(FirstPersonScale == OtherInfo.FirstPersonScale) &&
		(OrthoWidth == OtherInfo.OrthoWidth) &&
		(OrthoNearClipPlane == OtherInfo.OrthoNearClipPlane) &&
		(OrthoFarClipPlane == OtherInfo.OrthoFarClipPlane) &&
		((PerspectiveNearClipPlane == OtherInfo.PerspectiveNearClipPlane) || //either they are the same or both don't override
			(PerspectiveNearClipPlane <= 0.f && OtherInfo.PerspectiveNearClipPlane <= 0.f)) &&
		(AspectRatio == OtherInfo.AspectRatio) &&
		(bConstrainAspectRatio == OtherInfo.bConstrainAspectRatio) &&
		(bUseFirstPersonParameters == OtherInfo.bUseFirstPersonParameters) &&
		(bUseFieldOfViewForLOD == OtherInfo.bUseFieldOfViewForLOD) &&
		(ProjectionMode == OtherInfo.ProjectionMode) &&
		(OffCenterProjectionOffset == OtherInfo.OffCenterProjectionOffset);
}

void FMinimalViewInfo::BlendViewInfo(FMinimalViewInfo& OtherInfo, float OtherWeight)
{
	Location = FMath::Lerp(Location, OtherInfo.Location, OtherWeight);

	const FRotator DeltaAng = (OtherInfo.Rotation - Rotation).GetNormalized();
	Rotation = Rotation + OtherWeight * DeltaAng;

	FOV = FMath::Lerp(FOV, OtherInfo.FOV, OtherWeight);
	FirstPersonFOV = FMath::Lerp(FirstPersonFOV, OtherInfo.FirstPersonFOV, OtherWeight);
	FirstPersonScale = FMath::Lerp(FirstPersonScale, OtherInfo.FirstPersonScale, OtherWeight);
	OrthoWidth = FMath::Lerp(OrthoWidth, OtherInfo.OrthoWidth, OtherWeight);
	OrthoNearClipPlane = FMath::Lerp(OrthoNearClipPlane, OtherInfo.OrthoNearClipPlane, OtherWeight);
	OrthoFarClipPlane = FMath::Lerp(OrthoFarClipPlane, OtherInfo.OrthoFarClipPlane, OtherWeight);
	PerspectiveNearClipPlane = FMath::Lerp(PerspectiveNearClipPlane, OtherInfo.PerspectiveNearClipPlane, OtherWeight);
	OffCenterProjectionOffset = FMath::Lerp(OffCenterProjectionOffset, OtherInfo.OffCenterProjectionOffset, OtherWeight);

	AspectRatio = FMath::Lerp(AspectRatio, OtherInfo.AspectRatio, OtherWeight);
	bConstrainAspectRatio |= OtherInfo.bConstrainAspectRatio;
	bUseFirstPersonParameters |= OtherInfo.bUseFirstPersonParameters;
	bUseFieldOfViewForLOD |= OtherInfo.bUseFieldOfViewForLOD;
}

void FMinimalViewInfo::ApplyBlendWeight(const float& Weight)
{
	Location *= Weight;
	Rotation.Normalize();
	Rotation *= Weight;
	FOV *= Weight;
	FirstPersonFOV *= Weight;
	FirstPersonScale *= Weight;
	OrthoWidth *= Weight;
	OrthoNearClipPlane *= Weight;
	OrthoFarClipPlane *= Weight;
	PerspectiveNearClipPlane *= Weight;
	AspectRatio *= Weight;
	OffCenterProjectionOffset *= Weight;
}

void FMinimalViewInfo::AddWeightedViewInfo(const FMinimalViewInfo& OtherView, const float& Weight)
{
	FMinimalViewInfo OtherViewWeighted = OtherView;
	OtherViewWeighted.ApplyBlendWeight(Weight);

	Location += OtherViewWeighted.Location;
	Rotation += OtherViewWeighted.Rotation;
	FOV += OtherViewWeighted.FOV;
	FirstPersonFOV += OtherViewWeighted.FirstPersonFOV;
	FirstPersonScale += OtherViewWeighted.FirstPersonScale;
	OrthoWidth += OtherViewWeighted.OrthoWidth;
	OrthoNearClipPlane += OtherViewWeighted.OrthoNearClipPlane;
	OrthoFarClipPlane += OtherViewWeighted.OrthoFarClipPlane;
	PerspectiveNearClipPlane += OtherViewWeighted.PerspectiveNearClipPlane;
	AspectRatio += OtherViewWeighted.AspectRatio;
	OffCenterProjectionOffset += OtherViewWeighted.OffCenterProjectionOffset;

	bConstrainAspectRatio |= OtherViewWeighted.bConstrainAspectRatio;
	bUseFirstPersonParameters |= OtherViewWeighted.bUseFirstPersonParameters;
	bUseFieldOfViewForLOD |= OtherViewWeighted.bUseFieldOfViewForLOD;
}

FMatrix FMinimalViewInfo::CalculateProjectionMatrix() const
{
	FMatrix ProjectionMatrix;

	const bool bOrthographic = ProjectionMode == ECameraProjectionMode::Orthographic;
	if (bOrthographic)
	{
		const float YScale = 1.0f / AspectRatio;

		const float HalfOrthoWidth = OrthoWidth / 2.0f;
		const float ScaledOrthoHeight = OrthoWidth / 2.0f * YScale;

		const float NearPlane = OrthoNearClipPlane;
		const float FarPlane = OrthoFarClipPlane;

		const float ZScale = 1.0f / (FarPlane - NearPlane);
		const float ZOffset = -NearPlane;

		ProjectionMatrix = FReversedZOrthoMatrix(
			HalfOrthoWidth,
			ScaledOrthoHeight,
			ZScale,
			ZOffset
			);
	}
	else
	{
		const float ClippingPlane = GetFinalPerspectiveNearClipPlane();
		// Avoid divide by zero in the projection matrix calculation by clamping FOV
		ProjectionMatrix = FReversedZPerspectiveMatrix(
			FMath::Max(0.001f, FOV) * (float)UE_PI / 360.0f,
			AspectRatio,
			1.0f,
			ClippingPlane);
	}

	if (!OffCenterProjectionOffset.IsZero())
	{
		const float Left = -1.0f + OffCenterProjectionOffset.X;
		const float Right = Left + 2.0f;
		const float Bottom = -1.0f + OffCenterProjectionOffset.Y;
		const float Top = Bottom + 2.0f;

		// Make sure you update CalculateProjectionMatrixGivenViewRectangle(...) as well if you change this, as
		// it may have already modified some fields in the ProjectionMatrix.
		if (bOrthographic)
		{
			ProjectionMatrix.M[3][0] = (Left + Right) / (Left - Right);
			ProjectionMatrix.M[3][1] = (Bottom + Top) / (Bottom - Top);
		}
		else
		{
			ProjectionMatrix.M[2][0] = (Left + Right) / (Left - Right);
			ProjectionMatrix.M[2][1] = (Bottom + Top) / (Bottom - Top);
		}
	}

	return ProjectionMatrix;
}

void FMinimalViewInfo::CalculateProjectionMatrixGivenViewRectangle(FMinimalViewInfo& ViewInfo, TEnumAsByte<enum EAspectRatioAxisConstraint> AspectRatioAxisConstraint, const FIntRect& ConstrainedViewRectangle, FSceneViewProjectionData& InOutProjectionData)
{
#if !(UE_BUILD_SHIPPING)
	if (CVarDebugForceAllCamerasToOrtho.GetValueOnAnyThread())
	{
		ViewInfo.ProjectionMode = ECameraProjectionMode::Orthographic;
		ViewInfo.OrthoWidth = CVarDebugForceCameraOrthoWidth.GetValueOnAnyThread();
		ViewInfo.bAutoCalculateOrthoPlanes = CVarDebugForceUseOrthoAutoPlanes.GetValueOnAnyThread();
		ViewInfo.OrthoNearClipPlane = CVarDebugForceCameraOrthoNearPlane.GetValueOnAnyThread();
		ViewInfo.OrthoFarClipPlane = CVarDebugForceCameraOrthoFarPlane.GetValueOnAnyThread();
	}
#endif

	bool bOrthographic = ViewInfo.ProjectionMode == ECameraProjectionMode::Orthographic;
	if(bOrthographic)
	{
		ViewInfo.AutoCalculateOrthoPlanes(InOutProjectionData);
	}

	// Create the projection matrix (and possibly constrain the view rectangle)
	if (ViewInfo.bConstrainAspectRatio)
	{
		// Enforce a particular aspect ratio for the render of the scene. 
		// Results in black bars at top/bottom etc.
		InOutProjectionData.SetConstrainedViewRectangle(ConstrainedViewRectangle);
		if(bOrthographic)
		{
			InOutProjectionData.UpdateOrthoPlanes(ViewInfo);
		}
		InOutProjectionData.ProjectionMatrix = ViewInfo.CalculateProjectionMatrix();
	}
	else
	{
		float XAxisMultiplier;
		float YAxisMultiplier;

		const FIntRect& ViewRect = InOutProjectionData.GetViewRect();
		const int32 SizeX = ViewRect.Width();
		const int32 SizeY = ViewRect.Height();

		// Get effective aspect ratio axis constraint.
		AspectRatioAxisConstraint = ViewInfo.AspectRatioAxisConstraint.Get(AspectRatioAxisConstraint);

		// If x is bigger, and we're respecting x or major axis, AND mobile isn't forcing us to be Y axis aligned
		const bool bMaintainXFOV = 
			((SizeX > SizeY) && (AspectRatioAxisConstraint == AspectRatio_MajorAxisFOV)) ||
			(AspectRatioAxisConstraint == AspectRatio_MaintainXFOV);
		if (bMaintainXFOV)
		{
			// If the viewport is wider than it is tall
			XAxisMultiplier = 1.0f;
			YAxisMultiplier = SizeX / (float)SizeY;
		}
		else
		{
			// If the viewport is taller than it is wide
			XAxisMultiplier = SizeY / (float)SizeX;
			YAxisMultiplier = 1.0f;
		}
		
		if (bOrthographic)
		{
			const float OrthoWidth = (ViewInfo.OrthoWidth / 2.0f) / XAxisMultiplier;
			const float OrthoHeight = (ViewInfo.OrthoWidth / 2.0f) / YAxisMultiplier;

			float FarPlane = ViewInfo.OrthoFarClipPlane;
			float NearPlane = ViewInfo.OrthoNearClipPlane;

			InOutProjectionData.UpdateOrthoPlanes(NearPlane, FarPlane, OrthoWidth, ViewInfo.bUseCameraHeightAsViewTarget);

			const float ZScale = 1.0f / (FarPlane - NearPlane);
			const float ZOffset = -NearPlane;

			InOutProjectionData.ProjectionMatrix = FReversedZOrthoMatrix(
				OrthoWidth, 
				OrthoHeight,
				ZScale,
				ZOffset
				);		
		}
		else
		{
			float MatrixHalfFOV;
			if (!bMaintainXFOV && ViewInfo.AspectRatio != 0.f && !CVarUseLegacyMaintainYFOV.GetValueOnGameThread())
			{
				// The view-info FOV is horizontal. But if we have a different aspect ratio constraint, we need to
				// adjust this FOV value using the aspect ratio it was computed with, so we that we can compute the
				// complementary FOV value (with the *effective* aspect ratio) correctly.
				const float HalfXFOV = FMath::DegreesToRadians(FMath::Max(0.001f, ViewInfo.FOV) / 2.f);
				const float HalfYFOV = FMath::Atan(FMath::Tan(HalfXFOV) / ViewInfo.AspectRatio);
				MatrixHalfFOV = HalfYFOV;
			}
			else
			{
				// Avoid divide by zero in the projection matrix calculation by clamping FOV.
				// Note the division by 360 instead of 180 because we want the half-FOV.
				MatrixHalfFOV = FMath::Max(0.001f, ViewInfo.FOV) * (float)UE_PI / 360.0f;
			}

			const float ClippingPlane = ViewInfo.GetFinalPerspectiveNearClipPlane();
			InOutProjectionData.ProjectionMatrix = FReversedZPerspectiveMatrix(
				MatrixHalfFOV,
				MatrixHalfFOV,
				XAxisMultiplier,
				YAxisMultiplier,
				ClippingPlane,
				ClippingPlane
			);
		}
	}

	if (!ViewInfo.OffCenterProjectionOffset.IsZero())
	{
		const float Left = -1.0f + ViewInfo.OffCenterProjectionOffset.X;
		const float Right = Left + 2.0f;
		const float Bottom = -1.0f + ViewInfo.OffCenterProjectionOffset.Y;
		const float Top = Bottom + 2.0f;

		// Make sure you update CalculateProjectionMatrix() as well if you change this, as
		// it may have already modified some fields in the ProjectionMatrix.
		if (bOrthographic)
		{
			InOutProjectionData.ProjectionMatrix.M[3][0] = (Left + Right) / (Left - Right);
			InOutProjectionData.ProjectionMatrix.M[3][1] = (Bottom + Top) / (Bottom - Top);
		}
		else
		{
			InOutProjectionData.ProjectionMatrix.M[2][0] = (Left + Right) / (Left - Right);
			InOutProjectionData.ProjectionMatrix.M[2][1] = (Bottom + Top) / (Bottom - Top);
		}
	}
}

void FMinimalViewInfo::CalculateProjectionMatrixGivenView(FMinimalViewInfo& ViewInfo, TEnumAsByte<enum EAspectRatioAxisConstraint> AspectRatioAxisConstraint, FViewport* Viewport, FSceneViewProjectionData& InOutProjectionData)
{
	// Factor in any asymmetric crop, which can change the output aspect ratio
	const float CropAspectRatio = (ViewInfo.AsymmetricCropFraction.X + ViewInfo.AsymmetricCropFraction.Y) / (ViewInfo.AsymmetricCropFraction.Z + ViewInfo.AsymmetricCropFraction.W);
	const float AspectRatio = ViewInfo.AspectRatio * CropAspectRatio;

	FIntRect ViewExtents = Viewport->CalculateViewExtents(AspectRatio, InOutProjectionData.GetViewRect());
	CalculateProjectionMatrixGivenViewRectangle(ViewInfo, AspectRatioAxisConstraint, ViewExtents, InOutProjectionData);
}

bool FMinimalViewInfo::AutoCalculateOrthoPlanes(FSceneViewProjectionData& InOutProjectionData)
{	
	if (ProjectionMode == ECameraProjectionMode::Orthographic && CVarOrthoAllowAutoPlanes.GetValueOnAnyThread() && bAutoCalculateOrthoPlanes)
	{
		//First check if we are using 16bit buffer and unit scaling, then set the min/max values accordingly
		const bool bUse16bitDepth = CVarOrthoClampToMaxFPBuffer.GetValueOnAnyThread() == 1;
		const bool bScaleIncrementingUnits = CVarOrthoScaleIncrementingUnits.GetValueOnAnyThread() && bUse16bitDepth;
		const float MaxFPValue = bScaleIncrementingUnits ? UE_LARGE_WORLD_MAX : UE_OLD_WORLD_MAX;
		float FPScale = bUse16bitDepth ? 65504.0f : UE_OLD_WORLD_MAX;

		const float AutoDepthScale = CVarOrthoAutoDepthScale.GetValueOnAnyThread();
		if (AutoDepthScale > 0.0f)
		{
			//This allows the user to override the FP scaling value, where the default is 16bit.
			FPScale = FMath::Clamp(AutoDepthScale, 1.0f, FPScale);
		}

		//Get the OrthoHeight, with Ortho the depth is typically bound to the Y axis so we use that
		const float OrthoHeight = OrthoWidth / (AspectRatio == 0.0f ? UE_DELTA : AspectRatio);

		//Get the normalized view forward vector of the camera
		const FRotationMatrix RotMat(Rotation);
		FVector ViewForward = RotMat.GetColumn(2);
		ViewForward.Normalize();

		/**
		 * The CosAngle is the cosine of the angle between the ViewForward and camera down.
		 * Forcing the absolute value for this means that Up/Down is 1.0f and Forward (90 degrees) is 0.
		 * We use this to scale the Near Plane, and the far plane if 16 bit scaling is disabled.
		 */
		float CosAngle = FMath::Abs(ViewForward.Z);
		
		/** 
		 * We still max out at UE_OLD_WORLD_MAX or Max32FP, but we scale the FarPlane depending on ratio of the pixel size to the world unit size.
		 * Details below, but the reasoning is, we can't visibly see smaller than a pixel, 
		 * so the passes that need 16bit buffers such as HZB have their plane distances scaled automatically depending on this ratio.
		 */		
		float FarPlane;
		float UnitPerPixelRatio = 1.0f;
		const FIntRect& ViewportSize = InOutProjectionData.GetViewRect();
		if (bUse16bitDepth && OrthoHeight > 0  && ViewportSize.Area() > 0)
		{
			//The CmPerPixelRatio determines the far plane depth scale required for the scene
			UnitPerPixelRatio = FMath::FloorToFloat(OrthoHeight / (float)ViewportSize.Height());
			if (bScaleIncrementingUnits)
			{	
				//This scales the min/max depending on the dynamic scale of the unit to pixel as the ortho width increases at the sacrifice of the max FarPlane, allowing scaling to LWC
				UnitPerPixelRatio = FMath::Log2(UnitPerPixelRatio);
			}
			
			FarPlane = FMath::Clamp(FPScale * UnitPerPixelRatio, FPScale, MaxFPValue);
		}
		else
		{
			/** 
			 * Default path if the 16bit scaling depth is disabled or not usable.
			 * Note: this path does not scale for 16 bit buffers, it only calculates Near/Far plane min/max automatically
			 */			
			FarPlane = OrthoHeight / (CosAngle == 0.0f ? UE_DELTA : CosAngle);
		}
		/** 
		* The camera arm length is adjusted depending on the CosAngle as the horizontal view typically has a significantly larger plane range, 
		* so it becomes irrelevant, whereas it is necessary to account for in a top down view. Note: a small scene camera arm length will become irrelevant for a large ortho width.
		*/
		float CameraArmLength = static_cast<float>(CameraToViewTarget.Length()) * CosAngle;

		/**
		 * The NearPlane calculation is a scaled OrthoHeight depending on the camera angle, 
		 * which maxes out at 45 degrees by default as this captures the entire scene for the majority of angles.
		 * r.Ortho.AutoPlanes.ShiftPlanes should be used to account for views outside of this.
		 * 
		 * The FarPlane is the required depth precision interpretation for the UnitPerPixelRatio. 
		 * We clamp this to remove the Near plane difference, and also max out at the previously set maximum FPValue. 
		 * This setup should help for possible future implementations where we can increase the depth range (i.e. LWC + double float depth buffers).
		 */
		float SinAngle = FMath::Clamp(1.0f - CosAngle, 0.707107f, 1.0f);
		float NearPlane = FMath::Max(OrthoWidth, OrthoHeight) * FMath::Max((FMath::Clamp(CosAngle, 0.707107f, 1.0f) - (1.0f/SinAngle)), -0.5f) - CameraArmLength;
		FarPlane = FMath::Clamp(FarPlane, OrthoHeight, MaxFPValue + NearPlane);

		//The Planes can be scaled in the Z axis without restriction to ensure a user can capture their entire view.
		const float GlobalAutoPlaneShift = CVarOrthoAutoPlaneShift.GetValueOnAnyThread();
		OrthoNearClipPlane = NearPlane + AutoPlaneShift + GlobalAutoPlaneShift;
		OrthoFarClipPlane = FarPlane + AutoPlaneShift + GlobalAutoPlaneShift;
		InOutProjectionData.CameraToViewTarget = CameraToViewTarget;
		return true;
	}
	return false;
}

FVector FMinimalViewInfo::TransformWorldToFirstPerson(const FVector& WorldPosition, bool bIgnoreFirstPersonScale) const
{
	if (ProjectionMode == ECameraProjectionMode::Perspective)
	{
		const FVector Forward = Rotation.Vector();
		const FVector CameraRelativePosition = WorldPosition - Location;
		const FVector ProjectedPosition = FVector::DotProduct(Forward, CameraRelativePosition) * Forward;
		const FVector Rejection = CameraRelativePosition - ProjectedPosition;
		const float FOVCorrectionFactor = CalculateFirstPersonFOVCorrectionFactor();
		const FVector FOVCorrectedPosition = ProjectedPosition + Rejection * FOVCorrectionFactor;
		const FVector ScaledPosition = FOVCorrectedPosition * FirstPersonScale;
		const FVector Result = (bIgnoreFirstPersonScale ? FOVCorrectedPosition : ScaledPosition) + Location;
		return Result;
	}
	return WorldPosition;
}

FVector FMinimalViewInfo::TransformFirstPersonToWorld(const FVector& FirstPersonPosition, bool bScaleWasIgnoredOnInput) const
{
	if (ProjectionMode == ECameraProjectionMode::Perspective)
	{
		const FVector Forward = Rotation.Vector();
		const FVector CameraRelativePosition = FirstPersonPosition - Location;
		const FVector UnscaledPosition = bScaleWasIgnoredOnInput || FMath::Abs(FirstPersonScale) < UE_SMALL_NUMBER ? CameraRelativePosition : CameraRelativePosition / FirstPersonScale;
		const FVector ProjectedPosition = FVector::DotProduct(Forward, UnscaledPosition) * Forward;
		const FVector Rejection = UnscaledPosition - ProjectedPosition;
		const float FOVCorrectionFactor = CalculateFirstPersonFOVCorrectionFactor();
		const FVector FOVUncorrectedPosition = FMath::Abs(FOVCorrectionFactor) > UE_SMALL_NUMBER ? (ProjectedPosition + (Rejection / FOVCorrectionFactor)) : UnscaledPosition;
		const FVector Result = FOVUncorrectedPosition + Location;
		return Result;
	}
	return FirstPersonPosition;
}

float FMinimalViewInfo::CalculateFirstPersonFOVCorrectionFactor() const
{
	const float HalfTanSceneFOV = FMath::Tan(FMath::DegreesToRadians(FOV) * 0.5f);
	const float HalfTanFirstPersonFOV = FMath::Tan(FMath::DegreesToRadians(FirstPersonFOV * 0.5f));
	const float FOVCorrectionFactor = HalfTanSceneFOV / HalfTanFirstPersonFOV;
	return FOVCorrectionFactor;
}

void FMinimalViewInfo::ApplyOverscan(float InOverscan, bool bScaleResolutionWithOverscan, bool bCropOverscan)
{
	if (!FMath::IsNearlyZero(InOverscan))
	{
		// Clamp the incoming overscan so that the new total overscan can never be less than zero
		const float ClampedOverscan = FMath::Max(InOverscan, -Overscan / (1 + Overscan));
		
		// Keep track of the total amount of overscan that has been applied to the view. Mathematically,
		// this formula is derived from 1 + TotalOverscan = (1 + Overscan) * (1 + InOverscan)
		Overscan = Overscan * (1.0f + ClampedOverscan) + ClampedOverscan;
		
		// By convention, 0.0 means no overscan, so add 1 to compute the scalar needed for altering projection values
		const float OverscanScalar = 1.0f + ClampedOverscan;
		
		// Overscan directly scales the view frustum, but can be accomplished by scaling the FOV.
		// However, must scale the tangent of the half-FOV to accomplish the same mathematical transform.
		const float HalfFOVInRadians = FMath::DegreesToRadians(0.5f * FOV);
		const float OverscannedFOV = FMath::Atan(OverscanScalar * FMath::Tan(HalfFOVInRadians));
		FOV = 2.0f * FMath::RadiansToDegrees(OverscannedFOV);
		
		OrthoWidth *= OverscanScalar;

		if (bScaleResolutionWithOverscan)
		{
			// Ensure that the resolution fraction stays between 1.0 and 2.0
			OverscanResolutionFraction = FMath::Clamp(OverscanResolutionFraction * OverscanScalar, 1.0f, 2.0f);
		}

		CropFraction *= bCropOverscan ? 1.0f / OverscanScalar : 1.0f;
	}
}

void FMinimalViewInfo::ApplyAsymmetricOverscan(const FVector4f& InAsymmetricOverscan, bool bScaleResolutionWithOverscan, bool bCropOverscan)
{
	if (!FMath::IsNearlyZero(InAsymmetricOverscan.Size()))
	{
		// Clamp the incoming overscan so that the new total overscan can never be less than zero
		const FVector4f InverseAsymmetricOverscan = FVector4f(
			(AsymmetricOverscan.Y - AsymmetricOverscan.X + 2.0f) / (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f) - 1.0f,
			(AsymmetricOverscan.X - AsymmetricOverscan.Y + 2.0f) / (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f) - 1.0f,
			(AsymmetricOverscan.W - AsymmetricOverscan.Z + 2.0f) / (AsymmetricOverscan.Z + AsymmetricOverscan.W + 2.0f) - 1.0f,
			(AsymmetricOverscan.Z - AsymmetricOverscan.W + 2.0f) / (AsymmetricOverscan.Z + AsymmetricOverscan.W + 2.0f) - 1.0f);
		
		const FVector4f ClampedAsymmetricOverscan = FVector4f(
			FMath::Max(InAsymmetricOverscan.X, InverseAsymmetricOverscan.X),
			FMath::Max(InAsymmetricOverscan.Y, InverseAsymmetricOverscan.Y),
			FMath::Max(InAsymmetricOverscan.Z, InverseAsymmetricOverscan.Z),
			FMath::Max(InAsymmetricOverscan.W, InverseAsymmetricOverscan.W));
	
		// Keep track of the total amount of asymmetric overscan that has been applied to the view. Mathematically,
		// this formula is derived from 1 + TotalOverscan = (1 + Overscan) * (1 + InOverscan)
		AsymmetricOverscan = AsymmetricOverscan * (ClampedAsymmetricOverscan + 1.0f) + ClampedAsymmetricOverscan;

		// By convention, 0.0 means no overscan, so add 1 to compute the scalar needed for altering projection values
		FVector4f AsymmetricOverscanScalar = ClampedAsymmetricOverscan + 1.0f;
		
		// Overscan directly scales the view frustum, but can be accomplished by scaling the FOV.
		// However, must scale the tangent of the half-FOV to accomplish the same mathematical transform.
		const float HalfFOVInRadians = FMath::DegreesToRadians(0.5f * FOV);
		const float OverscannedFOV = FMath::Atan(0.5f * (AsymmetricOverscanScalar.X + AsymmetricOverscanScalar.Y) * FMath::Tan(HalfFOVInRadians));
		FOV = 2.0f * FMath::RadiansToDegrees(OverscannedFOV);

		OrthoWidth *= 0.5f * (AsymmetricOverscanScalar.X + AsymmetricOverscanScalar.Y);
	
		AspectRatio *= (AsymmetricOverscanScalar.X + AsymmetricOverscanScalar.Y) / (AsymmetricOverscanScalar.Z + AsymmetricOverscanScalar.W);
		OffCenterProjectionOffset.X += (AsymmetricOverscanScalar.Y - AsymmetricOverscanScalar.X) / (AsymmetricOverscanScalar.X + AsymmetricOverscanScalar.Y);
		OffCenterProjectionOffset.Y += (AsymmetricOverscanScalar.Z - AsymmetricOverscanScalar.W) / (AsymmetricOverscanScalar.Z + AsymmetricOverscanScalar.W);

		FVector4f InvAsymmetricOverscanScalar = FVector4f(
			(AsymmetricOverscanScalar.Y - AsymmetricOverscanScalar.X + 2.0f) / (AsymmetricOverscanScalar.X + AsymmetricOverscanScalar.Y),
			(AsymmetricOverscanScalar.X - AsymmetricOverscanScalar.Y + 2.0f) / (AsymmetricOverscanScalar.X + AsymmetricOverscanScalar.Y),
			(AsymmetricOverscanScalar.W - AsymmetricOverscanScalar.Z + 2.0f) / (AsymmetricOverscanScalar.Z + AsymmetricOverscanScalar.W),
			(AsymmetricOverscanScalar.Z - AsymmetricOverscanScalar.W + 2.0f) / (AsymmetricOverscanScalar.Z + AsymmetricOverscanScalar.W));

		if (bScaleResolutionWithOverscan)
		{
			// Ensure that the resolution fraction stays between 1.0 and 2.0
			const float MaxResScale = FMath::Max(0.5f * (AsymmetricOverscanScalar.X + AsymmetricOverscanScalar.Y), 0.5f * (AsymmetricOverscanScalar.Z + AsymmetricOverscanScalar.W));
			OverscanResolutionFraction = FMath::Clamp(OverscanResolutionFraction * MaxResScale, 1.0f, 2.0f);
		}
		AsymmetricCropFraction *= bCropOverscan ? InvAsymmetricOverscanScalar : FVector4f::One();
	}
}

void FMinimalViewInfo::ClearOverscan()
{
	if (Overscan > 0.0f)
	{
		// Apply the inverse overscan to the view frustum to obtain the original frustum values (field of view, ortho width, etc)
		// Inverse overscan derived from (1 + Overscan) * (1 + InverseOverscan) = 1
		const float InverseOverscan = - Overscan / (1.0f + Overscan);
		ApplyOverscan(InverseOverscan);
	}
	
	if (AsymmetricOverscan.Size() > 0)
	{
		// Apply the inverse overscan to the view frustum to obtain the original frustum values (field of view, ortho width, etc)
		const FVector4f InverseAsymmetricOverscan = FVector4f(
			(AsymmetricOverscan.Y - AsymmetricOverscan.X + 2.0f) / (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f) - 1.0f,
			(AsymmetricOverscan.X - AsymmetricOverscan.Y + 2.0f) / (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f) - 1.0f,
			(AsymmetricOverscan.W - AsymmetricOverscan.Z + 2.0f) / (AsymmetricOverscan.Z + AsymmetricOverscan.W + 2.0f) - 1.0f,
			(AsymmetricOverscan.Z - AsymmetricOverscan.W + 2.0f) / (AsymmetricOverscan.Z + AsymmetricOverscan.W + 2.0f) - 1.0f);

		ApplyAsymmetricOverscan(InverseAsymmetricOverscan);
	}
	
	OverscanResolutionFraction = 1.0f;
	CropFraction = 1.0f;
	AsymmetricCropFraction = FVector4f::One();
}
