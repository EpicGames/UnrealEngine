// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/CameraBlockingVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraBlockingVolume)

ACameraBlockingVolume::ACameraBlockingVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionResponseToChannel(ECC_Camera, ECR_Block);
}

