// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/World.h"
#include "UObject/Package.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "RHI.h" // IWYU pragma: keep
#include "RHIContext.h"
#include "UObject/UObjectThreadContext.h"
#include "TextureResource.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CanvasRenderTarget2D)

UCanvasRenderTarget2D::UCanvasRenderTarget2D( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, World( nullptr )
	, SampleCount(ETextureRenderTargetSampleCount::RTSC_1)
{
	bNeedsTwoCopies = false;
	bShouldClearRenderTargetOnReceiveUpdate = true;
	bCanCreateUAV = true;
}


void UCanvasRenderTarget2D::UpdateResourceWithParams(EUpdateResourceFlags InFlags)
{
	if (SampleCount != ETextureRenderTargetSampleCount::RTSC_1)
	{
		bCanCreateUAV = false;
	}

	// Call parent implementation
	Super::UpdateResourceWithParams(InFlags);

	// Don't allocate canvas object for CRT2D CDO; also, we can't update it during PostLoad!
	if (IsTemplate() || FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return;
	}

	RepaintCanvas();
}

void UCanvasRenderTarget2D::FastUpdateResource()
{
	if (GetResource() == nullptr)
	{
		// We don't have a resource, just take the fast path
		UpdateResource();
	}
	else
	{
		// Don't allocate canvas object for CRT2D CDO
		if (IsTemplate())
		{
			return;
		}

		RepaintCanvas();
	}
}

ETextureRenderTargetSampleCount UCanvasRenderTarget2D::GetSampleCount() const
{
	return SampleCount;
}

void UCanvasRenderTarget2D::SetSampleCount(ETextureRenderTargetSampleCount InSampleCount)
{
	if (SampleCount != InSampleCount)
	{
		SampleCount = InSampleCount;

		OnSampleCountChanged();
	}
}

#if WITH_EDITOR
bool UCanvasRenderTarget2D::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName& Name = InProperty->GetFName();

	if (Name == GET_MEMBER_NAME_CHECKED(ThisClass, SampleCount))
	{
		return !bAutoGenerateMips;
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(ThisClass, bAutoGenerateMips))
	{
		return SampleCount == ETextureRenderTargetSampleCount::RTSC_1;
	}

	return true;
}

void UCanvasRenderTarget2D::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, SampleCount))
	{
		OnSampleCountChanged();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UCanvasRenderTarget2D::RepaintCanvas()
{
	// Create or find the canvas object to use to render onto the texture.  Multiple canvas render target textures can share the same canvas.
	static const FName CanvasName("CanvasRenderTarget2DCanvas");
	UCanvas* Canvas = (UCanvas*)StaticFindObjectFast(UCanvas::StaticClass(), GetTransientPackage(), CanvasName);
	if (Canvas == nullptr)
	{
		Canvas = NewObject<UCanvas>(GetTransientPackage(), CanvasName);
		Canvas->AddToRoot();
	}

	// Create the FCanvas which does the actual rendering.
	const UWorld* WorldPtr = World.Get();
	const ERHIFeatureLevel::Type FeatureLevel = WorldPtr != nullptr ? World->GetFeatureLevel() : GMaxRHIFeatureLevel;

	// NOTE: This texture may be null when this is invoked through blueprint from a cmdlet or server.
	FTextureRenderTarget2DResource* TextureRenderTarget = (FTextureRenderTarget2DResource*) GameThread_GetRenderTargetResource();

	FCanvas RenderCanvas(TextureRenderTarget, nullptr, FGameTime::GetTimeSinceAppStart(), FeatureLevel);
	Canvas->Init(GetSurfaceWidth(), GetSurfaceHeight(), nullptr, &RenderCanvas);

	if (TextureRenderTarget)
	{
		// Enqueue the rendering command to set up the rendering canvas.
		bool bClearRenderTarget = bShouldClearRenderTargetOnReceiveUpdate;
		ENQUEUE_RENDER_COMMAND(CanvasRenderTargetMakeCurrentCommand)(
			[TextureRenderTarget, bClearRenderTarget](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.Transition(FRHITransitionInfo(TextureRenderTarget->GetRenderTargetTexture(), ERHIAccess::Unknown, ERHIAccess::RTV));

			if (bClearRenderTarget)
			{
				FRHIRenderPassInfo RPInfo(TextureRenderTarget->GetRenderTargetTexture(), ERenderTargetActions::Clear_Store);
				RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearUCanvas"));
				RHICmdList.EndRenderPass();
			}
		});
	}

	if (IsValidChecked(this) && OnCanvasRenderTargetUpdate.IsBound())
	{
		OnCanvasRenderTargetUpdate.Broadcast(Canvas, GetSurfaceWidth(), GetSurfaceHeight());
	}

	ReceiveUpdate(Canvas, GetSurfaceWidth(), GetSurfaceHeight());

	// Clean up and flush the rendering canvas.
	Canvas->Canvas = nullptr;

	if (TextureRenderTarget)
	{
		RenderCanvas.Flush_GameThread();
	}

	UpdateResourceImmediate(false);
}

void UCanvasRenderTarget2D::OnSampleCountChanged()
{
	// UAVs are not supported for sample counts higher than one.
	bCanCreateUAV = (SampleCount == ETextureRenderTargetSampleCount::RTSC_1);

	if (bAutoGenerateMips && SampleCount != ETextureRenderTargetSampleCount::RTSC_1)
	{
		UE_LOGF(LogTexture, Warning, "Auto-generate mips is not supported with sample counts above 1, disabling.");
		bAutoGenerateMips = false;
	}
}

UCanvasRenderTarget2D* UCanvasRenderTarget2D::CreateCanvasRenderTarget2D(UObject* WorldContextObject, TSubclassOf<UCanvasRenderTarget2D> CanvasRenderTarget2DClass, int32 Width, int32 Height)
{
	if ((Width > 0) && (Height > 0) && (CanvasRenderTarget2DClass != NULL))
	{
		UCanvasRenderTarget2D* NewCanvasRenderTarget = NewObject<UCanvasRenderTarget2D>(GetTransientPackage(), CanvasRenderTarget2DClass);
		if (NewCanvasRenderTarget)
		{
			NewCanvasRenderTarget->World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
			NewCanvasRenderTarget->InitAutoFormat(Width, Height);
			return NewCanvasRenderTarget;
		}
	}

	return nullptr;
}

void UCanvasRenderTarget2D::GetSize(int32& Width, int32& Height)
{
	Width = GetSurfaceWidth();
	Height = GetSurfaceHeight();
}


UWorld* UCanvasRenderTarget2D::GetWorld() const
{
	return World.Get();
}

