// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Character.cpp: ACharacter implementation
=============================================================================*/

#include "GameFramework/Character.h"
#include "Animation/AnimMontage.h"
#include "Engine/World.h"
#include "GameFramework/DamageType.h"
#include "GameFramework/Controller.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/ArrowComponent.h"
#include "Engine/CollisionProfile.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Net/Core/PropertyConditions/PropertyConditions.h"
#include "Net/UnrealNetwork.h"
#include "DisplayDebugHelpers.h"
#include "Engine/Canvas.h"
#include "Animation/AnimInstance.h"
#include "Engine/DamageEvents.h"
#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Character)

DEFINE_LOG_CATEGORY_STATIC(LogCharacter, Log, All);

DECLARE_CYCLE_STAT(TEXT("Char OnNetUpdateSimulatedPosition"), STAT_CharacterOnNetUpdateSimulatedPosition, STATGROUP_Character);


FName ACharacter::MeshComponentName(TEXT("CharacterMesh0"));
FName ACharacter::CharacterMovementComponentName(TEXT("CharMoveComp"));
FName ACharacter::CapsuleComponentName(TEXT("CollisionCylinder"));


// CVars
namespace CharacterCVars
{
	// Allows characters to include acceleration in the data replicated to sim proxies
	int32 EnableCharacterAccelerationReplication = 0;
	static FAutoConsoleVariableRef CVarEnableCharacterAccelerationReplication(
		TEXT("p.EnableCharacterAccelerationReplication"),
		EnableCharacterAccelerationReplication,
		TEXT("Whether to author acceleration data with character movement replication to sim proxies."));

	static int32 UseLegacyDoJump = 1;
	static FAutoConsoleVariableRef CVarUseLegacyDoJump(
		TEXT("p.UseLegacyDoJump"),
		UseLegacyDoJump,
		TEXT("Should CheckJumpInput call the legacy DoJump or the new DoJump. Default is true while we are still deprecating the old DoJump"),
		ECVF_Default);
}

ACharacter::ACharacter(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName ID_Characters;
		FText NAME_Characters;
		FConstructorStatics()
			: ID_Characters(TEXT("Characters"))
			, NAME_Characters(NSLOCTEXT("SpriteCategory", "Characters", "Characters"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	// Character rotation only changes in Yaw, to prevent the capsule from changing orientation.
	// Ask the Controller for the full rotation if desired (ie for aiming).
	bUseControllerRotationPitch = false;
	bUseControllerRotationRoll = false;
	bUseControllerRotationYaw = true;

	CapsuleComponent = CreateDefaultSubobject<UCapsuleComponent>(ACharacter::CapsuleComponentName);
	CapsuleComponent->InitCapsuleSize(34.0f, 88.0f);
	CapsuleComponent->SetCollisionProfileName(UCollisionProfile::Pawn_ProfileName);

	CapsuleComponent->CanCharacterStepUpOn = ECB_No;
	CapsuleComponent->SetShouldUpdatePhysicsVolume(true);
	CapsuleComponent->SetCanEverAffectNavigation(false);
	CapsuleComponent->bDynamicObstacle = true;
	RootComponent = CapsuleComponent;

	bClientCheckEncroachmentOnNetUpdate = true;
	JumpKeyHoldTime = 0.0f;
	JumpMaxHoldTime = 0.0f;
    JumpMaxCount = 1;
    JumpCurrentCount = 0;
	JumpCurrentCountPreJump = 0;
    bWasJumping = false;

	SetAnimRootMotionTranslationScale(1.0f);

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("Arrow"));
	if (ArrowComponent)
	{
		ArrowComponent->ArrowColor = FColor(150, 200, 255);
		ArrowComponent->bTreatAsASprite = true;
		ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Characters;
		ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Characters;
		ArrowComponent->SetupAttachment(CapsuleComponent);
		ArrowComponent->bIsScreenSizeScaled = true;
		ArrowComponent->SetSimulatePhysics(false);
	}
#endif // WITH_EDITORONLY_DATA

	CharacterMovement = CreateDefaultSubobject<UCharacterMovementComponent>(ACharacter::CharacterMovementComponentName);
	if (CharacterMovement)
	{
		CharacterMovement->UpdatedComponent = CapsuleComponent;
	}

	RecalculateCrouchedEyeHeight();

	Mesh = CreateOptionalDefaultSubobject<USkeletalMeshComponent>(ACharacter::MeshComponentName);
	if (Mesh)
	{
		Mesh->AlwaysLoadOnClient = true;
		Mesh->AlwaysLoadOnServer = true;
		Mesh->bOwnerNoSee = false;
		Mesh->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPose;
		Mesh->bCastDynamicShadow = true;
		Mesh->bAffectDynamicIndirectLighting = true;
		Mesh->PrimaryComponentTick.TickGroup = TG_PrePhysics;
		Mesh->SetupAttachment(CapsuleComponent);
		static FName MeshCollisionProfileName("CharacterMesh");
		Mesh->SetCollisionProfileName(MeshCollisionProfileName);
		Mesh->SetGenerateOverlapEvents(false);
		Mesh->SetCanEverAffectNavigation(false);
	}

	BaseRotationOffset = FQuat::Identity;
	SetReplicatedGravityDirection(UCharacterMovementComponent::DefaultGravityDirection);
}

void ACharacter::PostInitializeComponents()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Character_PostInitComponents);

	Super::PostInitializeComponents();

	if (IsValidChecked(this))
	{
		if (Mesh)
		{
			CacheInitialMeshOffset(Mesh->GetRelativeLocation(), Mesh->GetRelativeRotation());

			// force animation tick after movement component updates
			if (Mesh->PrimaryComponentTick.bCanEverTick && CharacterMovement)
			{
				Mesh->PrimaryComponentTick.AddPrerequisite(CharacterMovement, CharacterMovement->PrimaryComponentTick);
			}
		}

		if (CharacterMovement && CapsuleComponent)
		{
			CharacterMovement->UpdateNavAgent(*CapsuleComponent);
		}

		if (GetController() == nullptr && GetNetMode() != NM_Client)
		{
			if (CharacterMovement && CharacterMovement->bRunPhysicsWithNoController)
			{
				CharacterMovement->SetDefaultMovementMode();
			}
		}
	}
}

void ACharacter::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (ArrowComponent)
	{
		ArrowComponent->SetSimulatePhysics(false);
	}
#endif // WITH_EDITORONLY_DATA
}

void ACharacter::BeginPlay()
{
	Super::BeginPlay();
}


void ACharacter::CacheInitialMeshOffset(FVector MeshRelativeLocation, FRotator MeshRelativeRotation)
{
	BaseTranslationOffset = MeshRelativeLocation;
	BaseRotationOffset = MeshRelativeRotation.Quaternion();

#if ENABLE_NAN_DIAGNOSTIC
	if (BaseRotationOffset.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("ACharacter::PostInitializeComponents detected NaN in BaseRotationOffset! (%s)"), *BaseRotationOffset.ToString());
	}

	const FRotator LocalRotation = Mesh->GetRelativeRotation();
	if (LocalRotation.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("ACharacter::PostInitializeComponents detected NaN in Mesh->RelativeRotation! (%s)"), *LocalRotation.ToString());
	}
#endif
}


UPawnMovementComponent* ACharacter::GetMovementComponent() const
{
	return CharacterMovement;
}

const FMovementBaseInterfaceData* ACharacter::GetMovementBaseInterfaceData() const
{
	return &BasedMovement.MovementBaseInterfaceData;
}

FMovementBaseInterfaceData* ACharacter::GetMovementBaseInterfaceData_Mutable()
{
	return &BasedMovement.MovementBaseInterfaceData;
}


void ACharacter::SetupPlayerInputComponent(UInputComponent* PlayerInputComponent)
{
	check(PlayerInputComponent);
}


void ACharacter::GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (IsTemplate())
	{
		UE_LOGF(LogCharacter, Log, "WARNING ACharacter::GetSimpleCollisionCylinder : Called on default object '%ls'. Will likely return zero size. Consider using GetDefaultHalfHeight() instead.", *this->GetPathName());
	}
#endif

	if (RootComponent == CapsuleComponent && IsRootComponentCollisionRegistered())
	{
		// Note: we purposefully ignore the component transform here aside from scale, always treating it as vertically aligned.
		// This improves performance and is also how we stated the CapsuleComponent would be used.
		CapsuleComponent->GetScaledCapsuleSize(CollisionRadius, CollisionHalfHeight);
	}
	else
	{
		Super::GetSimpleCollisionCylinder(CollisionRadius, CollisionHalfHeight);
	}
}

void ACharacter::UpdateNavigationRelevance()
{
	if (CapsuleComponent)
	{
		CapsuleComponent->SetCanEverAffectNavigation(bCanAffectNavigationGeneration);
	}
}

float ACharacter::GetDefaultHalfHeight() const
{
	UCapsuleComponent* DefaultCapsule = GetClass()->GetDefaultObject<ACharacter>()->CapsuleComponent;
	if (DefaultCapsule)
	{
		return DefaultCapsule->GetScaledCapsuleHalfHeight();
	}
	else
	{
		return Super::GetDefaultHalfHeight();
	}
}


UActorComponent* ACharacter::FindComponentByClass(const TSubclassOf<UActorComponent> ComponentClass) const
{
	// If the character has a Mesh, treat it as the first 'hit' when finding components
	if (Mesh && ComponentClass && Mesh->IsA(ComponentClass))
	{
		return Mesh;
	}

	return Super::FindComponentByClass(ComponentClass);
}

void ACharacter::OnWalkingOffLedge_Implementation(const FVector& PreviousFloorImpactNormal, const FVector& PreviousFloorContactNormal, const FVector& PreviousLocation, float TimeDelta)
{
}

void ACharacter::NotifyJumpApex()
{
	// Call delegate callback
	if (OnReachedJumpApex.IsBound())
	{
		OnReachedJumpApex.Broadcast();
	}
}

void ACharacter::Landed(const FHitResult& Hit)
{
	OnLanded(Hit);

	LandedDelegate.Broadcast(Hit);
}

bool ACharacter::CanJump() const
{
	return CanJumpInternal();
}

bool ACharacter::CanJumpInternal_Implementation() const
{
	return !IsCrouched() && JumpIsAllowedInternal();
}

bool ACharacter::JumpIsAllowedInternal() const
{
	// Ensure that the CharacterMovement state is valid
	bool bJumpIsAllowed = CharacterMovement->CanAttemptJump();

	if (bJumpIsAllowed)
	{
		// Ensure JumpHoldTime and JumpCount are valid.
		if (!bWasJumping || GetJumpMaxHoldTime() <= 0.0f)
		{
			// If this would be our first jump, but we're falling, normally we don't allow first jump to happen (we go right to second jump if available) 
			// One exception is if our subclass overrides and allows first jump to still happen while falling under certain conditions 
			if (JumpCurrentCount == 0 && CharacterMovement->IsFalling() && !CanJumpWhileFalling())
			{
				bJumpIsAllowed = JumpCurrentCount + 1 < JumpMaxCount;
			}
			else
			{
				bJumpIsAllowed = JumpCurrentCount < JumpMaxCount;
			}
		}
		else
		{
			// Only consider JumpKeyHoldTime as long as:
			// A) The jump limit hasn't been met OR
			// B) The jump limit has been met AND we were already jumping
			const bool bJumpKeyHeld = (bPressedJump && JumpKeyHoldTime < GetJumpMaxHoldTime());
			bJumpIsAllowed = bJumpKeyHeld &&
				((JumpCurrentCount < JumpMaxCount) || (bWasJumping && JumpCurrentCount == JumpMaxCount));
		}
	}

	return bJumpIsAllowed;
}

void ACharacter::SetProxyIsJumpForceApplied(const bool bInProxyIsJumpForceApplied)
{
	bProxyIsJumpForceApplied = bInProxyIsJumpForceApplied;
}

bool ACharacter::IsCrouched() const
{
	return bIsCrouched;
}

void ACharacter::SetIsCrouched(const bool bInIsCrouched)
{
	bIsCrouched = bInIsCrouched;
}

void ACharacter::ResetJumpState()
{
	bPressedJump = false;
	bWasJumping = false;
	JumpKeyHoldTime = 0.0f;
	JumpForceTimeRemaining = 0.0f;

	if (CharacterMovement && !CharacterMovement->IsFalling())
	{
		JumpCurrentCount = 0;
		JumpCurrentCountPreJump = 0;
	}
}

void ACharacter::OnJumped_Implementation()
{
}

bool ACharacter::IsJumpProvidingForce() const
{
	if (JumpForceTimeRemaining > 0.0f)
	{
		return true;
	}
	else if (GetProxyIsJumpForceApplied() && (GetLocalRole() == ROLE_SimulatedProxy))
	{
		return GetWorld()->TimeSince(ProxyJumpForceStartedTime) <= GetJumpMaxHoldTime();
	}

	return false;
}

void ACharacter::RecalculateBaseEyeHeight()
{
	if (!IsCrouched())
	{
		Super::RecalculateBaseEyeHeight();
	}
	else
	{
		BaseEyeHeight = CrouchedEyeHeight;
	}
}


void ACharacter::OnRep_IsCrouched()
{
	if (CharacterMovement)
	{
		if (IsCrouched())
		{
			CharacterMovement->bWantsToCrouch = true;
			CharacterMovement->Crouch(true);
		}
		else
		{
			CharacterMovement->bWantsToCrouch = false;
			CharacterMovement->UnCrouch(true);
		}
		CharacterMovement->bNetworkUpdateReceived = true;
	}
}

void ACharacter::SetProxyCharacterMovementActive(bool bInActive)
{
	bProxyIsCharacterMovementActive = bInActive;
}

void ACharacter::OnRep_IsCharacterMovementActive()
{
	if (CharacterMovement)
	{
		CharacterMovement->SetActiveFlag(bProxyIsCharacterMovementActive);
		CharacterMovement->OnRep_IsActive();
	}
}

void ACharacter::SetReplicateMovement(bool bInReplicateMovement)
{
	Super::SetReplicateMovement(bInReplicateMovement);

	if (CharacterMovement != nullptr && GetLocalRole() == ROLE_Authority)
	{
		// Set prediction data time stamp to current time to stop extrapolating
		// from time bReplicateMovement was turned off to when it was turned on again
		FNetworkPredictionData_Server* NetworkPrediction = CharacterMovement->HasPredictionData_Server() ? CharacterMovement->GetPredictionData_Server() : nullptr;

		if (NetworkPrediction != nullptr)
		{
			NetworkPrediction->ServerTimeStamp = GetWorld()->GetTimeSeconds();
		}
	}

	UpdateMeshAllowAutonomousTickPose();
}

bool ACharacter::CanCrouch() const
{
	return !IsCrouched() && CharacterMovement && CharacterMovement->CanEverCrouch() && GetRootComponent() && !GetRootComponent()->IsSimulatingPhysics();
}

void ACharacter::Crouch(bool bClientSimulation)
{
	if (CharacterMovement)
	{
		if (CanCrouch())
		{
			CharacterMovement->bWantsToCrouch = true;
		}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		else if (!CharacterMovement->CanEverCrouch())
		{
			UE_LOGF(LogCharacter, Log, "%ls is trying to crouch, but crouching is disabled on this character! (check CharacterMovement NavAgentSettings)", *GetName());
		}
#endif
	}
}

void ACharacter::UnCrouch(bool bClientSimulation)
{
	if (CharacterMovement)
	{
		CharacterMovement->bWantsToCrouch = false;
	}
}


void ACharacter::OnEndCrouch( float HeightAdjust, float ScaledHeightAdjust )
{
	RecalculateBaseEyeHeight();

	const ACharacter* DefaultChar = GetDefault<ACharacter>(GetClass());
	if (Mesh && DefaultChar->Mesh)
	{
		FVector& MeshRelativeLocation = Mesh->GetRelativeLocation_DirectMutable();
		MeshRelativeLocation.Z = DefaultChar->Mesh->GetRelativeLocation().Z;
		BaseTranslationOffset.Z = MeshRelativeLocation.Z;
	}
	else
	{
		BaseTranslationOffset.Z = DefaultChar->BaseTranslationOffset.Z;
	}

	K2_OnEndCrouch(HeightAdjust, ScaledHeightAdjust);
}

void ACharacter::OnStartCrouch( float HeightAdjust, float ScaledHeightAdjust )
{
	RecalculateBaseEyeHeight();

	const ACharacter* DefaultChar = GetDefault<ACharacter>(GetClass());
	if (Mesh && DefaultChar->Mesh)
	{
		FVector& MeshRelativeLocation = Mesh->GetRelativeLocation_DirectMutable();
		MeshRelativeLocation.Z = DefaultChar->Mesh->GetRelativeLocation().Z + HeightAdjust;
		BaseTranslationOffset.Z = MeshRelativeLocation.Z;
	}
	else
	{
		BaseTranslationOffset.Z = DefaultChar->BaseTranslationOffset.Z + HeightAdjust;
	}

	K2_OnStartCrouch(HeightAdjust, ScaledHeightAdjust);
}

void ACharacter::RecalculateCrouchedEyeHeight()
{
	if (CharacterMovement != nullptr)
	{
		constexpr float EyeHeightRatio = 0.8f;	// how high the character's eyes are, relative to the crouched height

		CrouchedEyeHeight = CharacterMovement->GetCrouchedHalfHeight() * EyeHeightRatio;
	}
}

void ACharacter::ApplyDamageMomentum(float DamageTaken, FDamageEvent const& DamageEvent, APawn* PawnInstigator, AActor* DamageCauser)
{
	UDamageType const* const DmgTypeCDO = DamageEvent.DamageTypeClass->GetDefaultObject<UDamageType>();
	float const ImpulseScale = DmgTypeCDO->DamageImpulse;

	if ( (ImpulseScale > 3.f) && (CharacterMovement != nullptr) )
	{
		FHitResult HitInfo;
		FVector ImpulseDir;
		DamageEvent.GetBestHitInfo(this, PawnInstigator, HitInfo, ImpulseDir);

		FVector Impulse = ImpulseDir * ImpulseScale;
		bool const bMassIndependentImpulse = !DmgTypeCDO->bScaleMomentumByMass;

		// limit Z momentum added if already going up faster than jump (to avoid blowing character way up into the sky)
		{
			FVector MassScaledImpulse = Impulse;
			if(!bMassIndependentImpulse && CharacterMovement->Mass > UE_SMALL_NUMBER)
			{
				MassScaledImpulse = MassScaledImpulse / CharacterMovement->Mass;
			}

			if ( (CharacterMovement->Velocity.Z > GetDefault<UCharacterMovementComponent>(CharacterMovement->GetClass())->JumpZVelocity) && (MassScaledImpulse.Z > 0.f) )
			{
				Impulse.Z *= 0.5f;
			}
		}

		CharacterMovement->AddImpulse(Impulse, bMassIndependentImpulse);
	}
}

void ACharacter::ClearCrossLevelReferences()
{
	if(BasedMovement.MovementBaseInterfaceData.IsValid())
	{
		const UObject* BodyInstanceOwnerSourceObject = BasedMovement.MovementBaseInterfaceData.GetMovementBaseObject();
		const UPackage* BodyInstanceOwnerOuterMost = BodyInstanceOwnerSourceObject ? BodyInstanceOwnerSourceObject->GetOutermost() : nullptr;
		if (GetOutermost() != BodyInstanceOwnerOuterMost)
		{
			FMovementBaseInterfaceData* DummyMovementBasePointer = nullptr;
			SetBase(DummyMovementBasePointer);
		}
	}

	Super::ClearCrossLevelReferences();
}

namespace MovementBaseUtility
{
	bool IsDynamicBase(const UPrimitiveComponent* MovementBase)
	{
		return (MovementBase && MovementBase->Mobility == EComponentMobility::Movable);
	}

	bool IsSimulatedBase(const UPrimitiveComponent* MovementBase)
	{
		bool bBaseIsSimulatingPhysics = false;
		const USceneComponent* AttachParent = MovementBase;
		while (!bBaseIsSimulatingPhysics && AttachParent)
		{
			bBaseIsSimulatingPhysics = AttachParent->IsSimulatingPhysics();
			AttachParent = AttachParent->GetAttachParent();
		}
		return bBaseIsSimulatingPhysics;
	}

	void AddTickDependency(FTickFunction& BasedObjectTick, UPrimitiveComponent* NewBase)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (NewBase && MovementBaseUtility::UseRelativeLocation(NewBase))
		{
			if (NewBase->PrimaryComponentTick.bCanEverTick)
			{
				BasedObjectTick.AddPrerequisite(NewBase, NewBase->PrimaryComponentTick);
			}

			AActor* NewBaseOwner = NewBase->GetOwner();
			if (NewBaseOwner)
			{
				if (NewBaseOwner->PrimaryActorTick.bCanEverTick)
				{
					BasedObjectTick.AddPrerequisite(NewBaseOwner, NewBaseOwner->PrimaryActorTick);
				}

				// @TODO: We need to find a more efficient way of finding all ticking components in an actor.
				for (UActorComponent* Component : NewBaseOwner->GetComponents())
				{
					// Dont allow a based component (e.g. a particle system) to push us into a different tick group
					if (Component && 
						Component->PrimaryComponentTick.bCanEverTick && 
						Component->PrimaryComponentTick.TickGroup <= BasedObjectTick.TickGroup && 
						Component->PrimaryComponentTick.EndTickGroup <= BasedObjectTick.TickGroup)
					{
						BasedObjectTick.AddPrerequisite(Component, Component->PrimaryComponentTick);
					}
				}
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void RemoveTickDependency(FTickFunction& BasedObjectTick, UPrimitiveComponent* OldBase)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (OldBase && MovementBaseUtility::UseRelativeLocation(OldBase))
		{
			BasedObjectTick.RemovePrerequisite(OldBase, OldBase->PrimaryComponentTick);
			AActor* OldBaseOwner = OldBase->GetOwner();
			if (OldBaseOwner)
			{
				BasedObjectTick.RemovePrerequisite(OldBaseOwner, OldBaseOwner->PrimaryActorTick);

				// @TODO: We need to find a more efficient way of finding all ticking components in an actor.
				for (UActorComponent* Component : OldBaseOwner->GetComponents())
				{
					if (Component && Component->PrimaryComponentTick.bCanEverTick)
					{
						BasedObjectTick.RemovePrerequisite(Component, Component->PrimaryComponentTick);
					}
				}
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FVector GetMovementBaseVelocity(const UPrimitiveComponent* MovementBase, const FName BoneName)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FVector BaseVelocity = FVector::ZeroVector;
		if (MovementBaseUtility::IsDynamicBase(MovementBase))
		{
			if (BoneName != NAME_None)
			{
				const FBodyInstance* BodyInstance = MovementBase->GetBodyInstance(BoneName);
				if (BodyInstance)
				{
					BaseVelocity = BodyInstance->GetUnrealWorldVelocity();
					return BaseVelocity;
				}
			}

			BaseVelocity = MovementBase->GetComponentVelocity();
			if (BaseVelocity.IsZero())
			{
				// Fall back to actor's Root component
				const AActor* Owner = MovementBase->GetOwner();
				if (Owner)
				{
					// Component might be moved manually (not by simulated physics or a movement component), see if the root component of the actor has a velocity.
					BaseVelocity = MovementBase->GetOwner()->GetVelocity();
				}				
			}

			// Fall back to physics velocity.
			if (BaseVelocity.IsZero())
			{
				if (FBodyInstance* BaseBodyInstance = MovementBase->GetBodyInstance())
				{
					BaseVelocity = BaseBodyInstance->GetUnrealWorldVelocity();
				}
			}
		}
		
		return BaseVelocity;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FVector GetMovementBaseTangentialVelocity(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& WorldLocation)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (MovementBaseUtility::IsDynamicBase(MovementBase))
		{
			if (const FBodyInstance* BodyInstance = MovementBase->GetBodyInstance(BoneName))
			{
				const FVector BaseAngVelInRad = BodyInstance->GetUnrealWorldAngularVelocityInRadians();
				if (!BaseAngVelInRad.IsNearlyZero())
				{
					FVector BaseLocation;
					FQuat BaseRotation;
					if (MovementBaseUtility::GetMovementBaseTransform(MovementBase, BoneName, BaseLocation, BaseRotation))
					{
						const FVector RadialDistanceToBase = WorldLocation - BaseLocation;
						const FVector TangentialVel = BaseAngVelInRad ^ RadialDistanceToBase;
						return TangentialVel;
					}
				}
			}			
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		return FVector::ZeroVector;
	}

	bool GetMovementBaseTransform(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector& OutLocation, FQuat& OutQuat)
	{
		if (MovementBase)
		{
			bool bGotTransformOfIntendedBone = true;

			if (BoneName != NAME_None)
			{
				// Check if this socket or bone exists (DoesSocketExist checks for either, as does requesting the transform).
				if (MovementBase->DoesSocketExist(BoneName))
				{
					MovementBase->GetSocketWorldLocationAndRotation(BoneName, OutLocation, OutQuat);
					return true;
				}
				else
				{
					UE_LOGF(LogCharacter, Warning, "GetMovementBaseTransform(): Invalid bone or socket '%ls' for PrimitiveComponent base %ls. Falling back to base's root transform.", *BoneName.ToString(), *GetPathNameSafe(MovementBase));
					bGotTransformOfIntendedBone = false;
				}
			}

			// No bone supplied (or it was invalid)
			OutLocation = MovementBase->GetComponentLocation();
			OutQuat = MovementBase->GetComponentQuat();
			return bGotTransformOfIntendedBone;
		}

		// nullptr MovementBase
		OutLocation = FVector::ZeroVector;
		OutQuat = FQuat::Identity;
		return false;
	}

	bool TransformLocationToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& LocalLocation, FVector& OutLocationWorldSpace)
	{
		FVector OutLocation;
		FQuat OutQuat;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bResult = GetMovementBaseTransform(MovementBase, BoneName, OutLocation, OutQuat);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OutLocationWorldSpace = FTransform(OutQuat, OutLocation).TransformPositionNoScale(LocalLocation);
		return bResult;
	}

	bool TransformLocationToLocal(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& WorldSpaceLocation, FVector& OutLocalLocation)
	{
		FVector OutLocation;
		FQuat OutQuat;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bResult = GetMovementBaseTransform(MovementBase, BoneName, OutLocation, OutQuat);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OutLocalLocation = FTransform(OutQuat, OutLocation).InverseTransformPositionNoScale(WorldSpaceLocation);
		return bResult;
	}

	bool TransformDirectionToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& LocalDirection, FVector& OutDirectionWorldSpace)
	{
		FVector IgnoredLocation;
		FQuat OutQuat;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bResult = GetMovementBaseTransform(MovementBase, BoneName, IgnoredLocation, OutQuat);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OutDirectionWorldSpace = OutQuat.RotateVector(LocalDirection);
		return bResult;
	}

	bool TransformDirectionToLocal(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& WorldSpaceDirection, FVector& OutLocalDirection)
	{
		FVector IgnoredLocation;
		FQuat OutQuat;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bResult = GetMovementBaseTransform(MovementBase, BoneName, IgnoredLocation, OutQuat);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OutLocalDirection = OutQuat.UnrotateVector(WorldSpaceDirection);
		return bResult;
	}

	bool IsDynamicBase(const FMovementBaseInterfaceData* MovementBase)
	{
		return (MovementBase && MovementBase->IsValid() && MovementBase->GetBodyInstanceOwner()->IsPhysicsOwnerMovable());
	}

	bool IsSimulatedBase(const FMovementBaseInterfaceData* MovementBase)
	{
		return (MovementBase && MovementBase->IsValid() && MovementBase->GetBodyInstanceOwner()->IsPhysicsOwnerSimulatingPhysics());
	}

	void AddTickDependency(FTickFunction& BasedObjectTick, FMovementBaseInterfaceData* MovementBase)
	{
		// Todo: Eventually convert this to not use PrimitiveComponents
		if (MovementBase)
		{
			UPrimitiveComponent* NewBase = Cast<UPrimitiveComponent>(MovementBase->PhysicsObjectOwner.Get());
		
			if (NewBase && MovementBaseUtility::UseRelativeLocation(MovementBase))
			{
				if (NewBase->PrimaryComponentTick.bCanEverTick)
				{
					BasedObjectTick.AddPrerequisite(NewBase, NewBase->PrimaryComponentTick);
				}

				AActor* NewBaseOwner = NewBase->GetOwner();
				if (NewBaseOwner)
				{
					if (NewBaseOwner->PrimaryActorTick.bCanEverTick)
					{
						BasedObjectTick.AddPrerequisite(NewBaseOwner, NewBaseOwner->PrimaryActorTick);
					}

					// @TODO: We need to find a more efficient way of finding all ticking components in an actor.
					for (UActorComponent* Component : NewBaseOwner->GetComponents())
					{
						// Dont allow a based component (e.g. a particle system) to push us into a different tick group
						if (Component && 
							Component->PrimaryComponentTick.bCanEverTick && 
							Component->PrimaryComponentTick.TickGroup <= BasedObjectTick.TickGroup && 
							Component->PrimaryComponentTick.EndTickGroup <= BasedObjectTick.TickGroup)
						{
							BasedObjectTick.AddPrerequisite(Component, Component->PrimaryComponentTick);
						}
					}
				}
			}
		}
	}

	void RemoveTickDependency(FTickFunction& BasedObjectTick, FMovementBaseInterfaceData* MovementBase)
	{
		// Todo: Eventually convert this to not use PrimitiveComponents
		if (MovementBase)
		{
			UPrimitiveComponent* OldBase = Cast<UPrimitiveComponent>(MovementBase->PhysicsObjectOwner.Get());
		
			if (OldBase && MovementBaseUtility::UseRelativeLocation(MovementBase))
			{
				BasedObjectTick.RemovePrerequisite(OldBase, OldBase->PrimaryComponentTick);
				AActor* OldBaseOwner = OldBase->GetOwner();
				if (OldBaseOwner)
				{
					BasedObjectTick.RemovePrerequisite(OldBaseOwner, OldBaseOwner->PrimaryActorTick);

					// @TODO: We need to find a more efficient way of finding all ticking components in an actor.
					for (UActorComponent* Component : OldBaseOwner->GetComponents())
					{
						if (Component && Component->PrimaryComponentTick.bCanEverTick)
						{
							BasedObjectTick.RemovePrerequisite(Component, Component->PrimaryComponentTick);
						}
					}
				}
			}
		}
	}

	FVector GetMovementBaseVelocity(const FMovementBaseInterfaceData* MovementBase, const FName BoneName)
	{
		FVector BaseVelocity = FVector::ZeroVector;
		if (IsDynamicBase(MovementBase))
		{
			if (BoneName != NAME_None)
			{
				if (const FBodyInstance* BodyInstance = MovementBase->GetBodyInstanceOwner()->GetBodyInstance(BoneName))
				{
					BaseVelocity = BodyInstance->GetUnrealWorldVelocity();
					return BaseVelocity;
				}
			}

			BaseVelocity = MovementBase->GetBodyInstanceOwner()->GetPhysicsOwnerVelocity();
			if (BaseVelocity.IsZero())
			{
				// Fall back to actor's Root component
				const AActor* Owner = Cast<const AActor>(MovementBase->GetMovementBaseObjectOwner());
				if (Owner)
				{
					// Component might be moved manually (not by simulated physics or a movement component), see if the root component of the actor has a velocity.
					BaseVelocity = Owner->GetVelocity();
				}
			}

			// Fall back to physics velocity.
			if (BaseVelocity.IsZero())
			{
				if (const FBodyInstance* BodyInstance = MovementBase->GetBodyInstanceOwner()->GetBodyInstance())
				{
					BaseVelocity = BodyInstance->GetUnrealWorldVelocity();
				}
			}
		}
		
		return BaseVelocity;
	}

	FVector GetMovementBaseTangentialVelocity(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& WorldLocation)
	{
		if (IsDynamicBase(MovementBase))
		{
			if (const FBodyInstance* BodyInstance = MovementBase->GetBodyInstanceOwner()->GetBodyInstance(BoneName))
			{
				const FVector BaseAngVelInRad = BodyInstance->GetUnrealWorldAngularVelocityInRadians();
				if (!BaseAngVelInRad.IsNearlyZero())
				{
					FVector BaseLocation;
					FQuat BaseRotation;
					if (GetMovementBaseTransform(MovementBase, BoneName, BaseLocation, BaseRotation))
					{
						const FVector RadialDistanceToBase = WorldLocation - BaseLocation;
						const FVector TangentialVel = BaseAngVelInRad ^ RadialDistanceToBase;
						return TangentialVel;
					}
				}
			}
			
			const FBodyInstance* BodyInstance = MovementBase->GetBodyInstanceOwner()->GetBodyInstance();
			const FVector BaseAngVelInRad = BodyInstance ? BodyInstance->GetUnrealWorldAngularVelocityInRadians() : FVector::ZeroVector;
			if (!BaseAngVelInRad.IsNearlyZero())
			{
				FVector BaseLocation;
				FQuat BaseRotation;
				if (GetMovementBaseTransform(MovementBase, BoneName, BaseLocation, BaseRotation))
				{
					const FVector RadialDistanceToBase = WorldLocation - BaseLocation;
					const FVector TangentialVel = BaseAngVelInRad ^ RadialDistanceToBase;
					return TangentialVel;
				}
			}
		}
		
		return FVector::ZeroVector;
	}

	bool GetMovementBaseTransform(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, FVector& OutLocation, FQuat& OutQuat)
	{
		if (MovementBase && MovementBase->IsValid())
		{
			bool bGotTransformOfIntendedBone = true;

			if (BoneName != NAME_None)
			{
				// Check if this socket or bone exists (DoesSocketExist checks for either, as does requesting the transform).
				if (MovementBase->GetBodyInstanceOwner()->DoesSocketExistOnPhysicsOwner(BoneName))
				{
					const FTransform BodyOwnerSocketTransform = MovementBase->GetBodyInstanceOwner()->GetPhysicsOwnerSocketTransform(BoneName);
					OutLocation = BodyOwnerSocketTransform.GetLocation();
					OutQuat = BodyOwnerSocketTransform.GetRotation();
					return true;
				}
				else
				{
					UE_LOGF(LogCharacter, Warning, "GetMovementBaseTransform(): Invalid bone or socket '%ls' for PrimitiveComponent base %ls. Falling back to base's root transform.", *BoneName.ToString(), *GetPathNameSafe(MovementBase->GetMovementBaseObject()));
					bGotTransformOfIntendedBone = false;
				}
			}
			
			// No bone supplied
			OutLocation = MovementBase->GetBodyInstanceOwner()->GetPhysicsOwnerTransform().GetLocation();
			OutQuat = MovementBase->GetBodyInstanceOwner()->GetPhysicsOwnerTransform().GetRotation();
			return bGotTransformOfIntendedBone;
		}

		// nullptr MovementBase
		OutLocation = FVector::ZeroVector;
		OutQuat = FQuat::Identity;
		return false;
	}

	bool TransformLocationToWorld(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& LocalLocation, FVector& OutLocationWorldSpace)
	{
		FVector OutLocation;
		FQuat OutQuat;
		const bool bResult = GetMovementBaseTransform(MovementBase, BoneName, OutLocation, OutQuat);
		OutLocationWorldSpace = FTransform(OutQuat, OutLocation).TransformPositionNoScale(LocalLocation);
		return bResult;
	}

	bool TransformLocationToLocal(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& WorldSpaceLocation, FVector& OutLocalLocation)
	{
		FVector OutLocation;
		FQuat OutQuat;
		const bool bResult = GetMovementBaseTransform(MovementBase, BoneName, OutLocation, OutQuat);
		OutLocalLocation = FTransform(OutQuat, OutLocation).InverseTransformPositionNoScale(WorldSpaceLocation);
		return bResult;
	}

	bool TransformDirectionToWorld(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& LocalDirection, FVector& OutDirectionWorldSpace)
	{
		FVector IgnoredLocation;
		FQuat OutQuat;
		const bool bResult = GetMovementBaseTransform(MovementBase, BoneName, IgnoredLocation, OutQuat);
		OutDirectionWorldSpace = OutQuat.RotateVector(LocalDirection);
		return bResult;
	}

	bool TransformDirectionToLocal(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& WorldSpaceDirection, FVector& OutLocalDirection)
	{
		FVector IgnoredLocation;
		FQuat OutQuat;
		const bool bResult = GetMovementBaseTransform(MovementBase, BoneName, IgnoredLocation, OutQuat);
		OutLocalDirection = OutQuat.UnrotateVector(WorldSpaceDirection);
		return bResult;
	}

	bool DoesMovementBaseDataMatch(const FMovementBaseInterfaceData* MovementBaseDataA, const FMovementBaseInterfaceData* MovementBaseDataB)
	{
		// if they're both null, they match
		if (MovementBaseDataA == nullptr && MovementBaseDataB == nullptr)
		{
			return true;
		}
		
		// if only one is null, they don't match
		if (MovementBaseDataA == nullptr || MovementBaseDataB == nullptr)
		{
			return false;
		}
		
		// Compare actual values
		if (MovementBaseDataA && MovementBaseDataB)
		{
			return *MovementBaseDataA == *MovementBaseDataB;
		}
		
		return false;
	}

	bool IsMovementBaseDataValid(const FMovementBaseInterfaceData* MovementBaseData)
	{
		return MovementBaseData && MovementBaseData->IsValid();
	}
	
	FMovementBaseInterfaceData GetMovementBaseDataFromPhysicsOwner(UObject* PhysicsObjectOwner)
	{
		/* TODO NS Movement Base Review Notes:
 		 * This is where we have to build up a IPhysicsBodyInstanceOwner interface from a PhysicsObjectOwner.
 		 * This is needed as IPhysicsBodyInstanceOwner isn't a UInterface so we need to instead replicate the PhysicsObjectOwner UObject and gather the rest of the info we need.
 		 * Which goes something like this:
 		 *  - First we need the Physics Handle (in order to get IPhysicsBodyInstanceOwner) - No good way to do this besides the IPhysicsComponent and UPrimitiveComponent route.
 		 *	  We also don't 100% need a physics handle. For instance, SG IPhysicsBodyInstanceOwnerResolver only has one body instance to point to so we just point to that rather
 		 *	  than using the handle to find something.
 		 *	- Now that we have the struct built we can use helper functions that use the struct or the IPhysicsBodyInstanceOwner directly through the struct
 		 *
 		 *	Overall, this is a lot of work to just build up the relevant data needed to make a MovementBaseData struct so I question if it's preferable. This can be avoided by
 		 *	just implementing functionality through a specific MovementBase UInterface but that requires adding that very specific interface to SG objects and Actor components
 		 *	so I want to balance complexity vs adding interfaces to solve every problem. IPhysicsBodyInstanceOwner avoids this problem as it's fairly broad and can be used
 		 *	for other things than just movement bases.
 		 *
 		 */
		FMovementBaseInterfaceData ReturnMovementBaseData;
		
		if (PhysicsObjectOwner)
		{
			IPhysicsBodyInstanceOwner* BodyInstanceOwner = nullptr;
			// TODO NS: Should we just implement IPhysicsBodyInstanceOwnerResolver on UPrimitiveComponent too? Avoid this specific check
			if (UPrimitiveComponent* PhysicsObjectOwnerAsPrimitiveComponent = Cast<UPrimitiveComponent>(PhysicsObjectOwner))
			{
				BodyInstanceOwner = PhysicsObjectOwnerAsPrimitiveComponent;
			}

			if (IPhysicsBodyInstanceOwnerResolver* PhysicsObjectOwnerResolver = Cast<IPhysicsBodyInstanceOwnerResolver>(PhysicsObjectOwner))
			{
				Chaos::FConstPhysicsObjectHandle PhysicsObject = nullptr;
				if (const IPhysicsComponent* PhysicsComponent = Cast<IPhysicsComponent>(PhysicsObjectOwner))
				{
					// Todo: Pass along some sort of ID so we can get something useful for objects owning multiple physics objects
					PhysicsObject = PhysicsComponent->GetPhysicsObjectById(0);
				}
				
				BodyInstanceOwner = PhysicsObjectOwnerResolver->ResolvePhysicsBodyInstanceOwner(PhysicsObject);
			}
			
			ReturnMovementBaseData.Set(PhysicsObjectOwner, BodyInstanceOwner);
		}
		
		return ReturnMovementBaseData;
	}

	FMovementBaseInterfaceData GetMovementBaseDataFromHitResult(const FHitResult* HitResult)
	{
		FMovementBaseInterfaceData ReturnMovementBaseData;

		if (HitResult)
		{
			const IPhysicsBodyInstanceOwner* BodyInstanceOwner = nullptr;
			// TODO NS: Should we just implement IPhysicsBodyInstanceOwnerResolver on UPrimitiveComponent too? Avoid this specific check
			if (UPrimitiveComponent* PhysicsObjectOwnerAsPrimitiveComponent = Cast<UPrimitiveComponent>(HitResult->PhysicsObjectOwner.Get()))
			{
				BodyInstanceOwner = PhysicsObjectOwnerAsPrimitiveComponent;
			}
		
			if (!BodyInstanceOwner)
			{
				BodyInstanceOwner = IPhysicsBodyInstanceOwner::GetPhysicsBodyInstanceOwnerFromHitResult(*HitResult);
			}
			
			ReturnMovementBaseData.Set(HitResult->PhysicsObjectOwner.Get(), BodyInstanceOwner);
			
			// Fall back to using UPrimitiveComponent Component in hit result if we didn't get anything valid from Physics Object and Component is set
			if (!ReturnMovementBaseData.IsValid() && HitResult->Component.IsValid())
			{
				ReturnMovementBaseData = FMovementBaseInterfaceData(HitResult->Component.Get());
			}
		}
		
		return ReturnMovementBaseData;
	}
}


/**	Change the Pawn's base. */
void ACharacter::SetBase( UPrimitiveComponent* NewBaseComponent, const FName InBoneName, bool bNotifyPawn )
{
	FMovementBaseInterfaceData MovementBaseInterfaceData(NewBaseComponent);
	SetBase(&MovementBaseInterfaceData, InBoneName, bNotifyPawn);
}

void ACharacter::SetBase(FMovementBaseInterfaceData* MovementBaseInterfaceData, const FName InBoneName, bool bNotifyPawn)
{
	// If NewBaseComponent is nullptr, ignore bone name.
	const FName BoneName = (MovementBaseInterfaceData ? InBoneName : NAME_None);

	// See what changed.
	const bool bBaseChanged = MovementBaseInterfaceData ? (MovementBaseInterfaceData->GetBodyInstanceOwner() != BasedMovement.MovementBaseInterfaceData.GetBodyInstanceOwner()) : BasedMovement.MovementBaseInterfaceData.IsValid();
	const bool bBoneChanged = (BoneName != BasedMovement.BoneName);

	if (bBaseChanged || bBoneChanged)
	{
		// Verify no recursion.
		if (MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid())
		{
			APawn* Loop = Cast<APawn>(MovementBaseInterfaceData->GetMovementBaseObjectOwner());
			while (Loop)
			{
				if (Loop == this)
				{
					UE_LOGF(LogCharacter, Warning, " SetBase failed! Recursion detected. Pawn %ls already based on %ls.", *GetName(), *MovementBaseInterfaceData->PhysicsObjectOwner->GetName());
					return;
				}
				
				bool bValidMovementBase = false;
				if (const FMovementBaseInterfaceData* LoopMovementBaseData = Loop->GetMovementBaseInterfaceData())
				{
					if (LoopMovementBaseData->IsValid())
					{
						Loop = Cast<APawn>(LoopMovementBaseData->GetMovementBaseObjectOwner());
						bValidMovementBase = true;
					}
				}
				
				if (!bValidMovementBase)
				{
					break;
				}
			}
		}

		// Set base.
		FMovementBaseInterfaceData OldMovementBaseInterfaceData = BasedMovement.MovementBaseInterfaceData;
		BasedMovement.BoneName = BoneName;
		if (MovementBaseInterfaceData)
		{
			BasedMovement.MovementBaseInterfaceData = *MovementBaseInterfaceData;
		}
		else
		{
			BasedMovement.MovementBaseInterfaceData.Clear();
		}
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BasedMovement.MovementBase = MovementBaseInterfaceData ? Cast<UPrimitiveComponent>(BasedMovement.MovementBaseInterfaceData.PhysicsObjectOwner) : nullptr;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		if (bBaseChanged)
		{
			// Increment base ID.
			BasedMovement.BaseID++;
		}

		if (CharacterMovement)
		{
			const bool bBaseIsSimulating = MovementBaseUtility::IsSimulatedBase(&BasedMovement.MovementBaseInterfaceData);
			if (bBaseChanged)
			{
				MovementBaseUtility::RemoveTickDependency(CharacterMovement->PrimaryComponentTick, &OldMovementBaseInterfaceData);
				// We use a special post physics function if simulating, otherwise add normal tick prereqs.
				if (!bBaseIsSimulating)
				{
					MovementBaseUtility::AddTickDependency(CharacterMovement->PrimaryComponentTick, &BasedMovement.MovementBaseInterfaceData);
				}
			}

			if (BasedMovement.MovementBaseInterfaceData.IsValid())
			{
				// Update OldBaseLocation/Rotation as those were referring to a different base
				// ... but not when handling replication for proxies (since they are going to copy this data from the replicated values anyway)
				if (!bInBaseReplication)
				{
					// Force base location and relative position to be computed since we have a new base or bone so the old relative offset is meaningless.
					CharacterMovement->SaveBaseLocation();
				}

				// Enable PostPhysics tick if we are standing on a physics object, as we need to to use post-physics transforms
				CharacterMovement->PostPhysicsTickFunction.SetTickFunctionEnable(bBaseIsSimulating);
			}
			else
			{
				BasedMovement.BoneName = NAME_None; // None, regardless of whether user tried to set a bone name, since we have no base component.
				BasedMovement.bRelativeRotation = false;
				CharacterMovement->CurrentFloor.Clear();
				CharacterMovement->PostPhysicsTickFunction.SetTickFunctionEnable(false);
			}

			const ENetRole LocalRole = GetLocalRole();
			const UObject* MovementBaseSourceObject = BasedMovement.MovementBaseInterfaceData.GetMovementBaseObject();
			if (LocalRole == ROLE_Authority || LocalRole == ROLE_AutonomousProxy)
			{
				BasedMovement.bServerHasBaseComponent = (BasedMovement.MovementBaseInterfaceData.GetBodyInstanceOwner() != nullptr); // Also set on autonomous proxies for nicer debugging.
				UE_LOGF(LogCharacter, Verbose, "Setting base on %ls for '%ls' to '%ls'", LocalRole == ROLE_Authority ? TEXT("Server") : TEXT("AutoProxy"), *GetName(), *GetFullNameSafe(MovementBaseSourceObject));
			}
			else
			{
				UE_LOGF(LogCharacter, Verbose, "Setting base on Client for '%ls' to '%ls'", *GetName(), *GetFullNameSafe(MovementBaseSourceObject));
			}

		}

		// Notify this actor of its new floor.
		if ( bNotifyPawn )
		{
			BaseChange();
		}
	}
}

void ACharacter::SetReplicatedServerLastTransformUpdateTimeStamp(float InReplicatedServerLastTransformUpdateTimeStamp)
{
	ReplicatedServerLastTransformUpdateTimeStamp = InReplicatedServerLastTransformUpdateTimeStamp; 
}

void ACharacter::SetReplayLastTransformUpdateTimeStamp(float InReplayLastTransformUpdateTimeStamp)
{
	ReplayLastTransformUpdateTimeStamp = InReplayLastTransformUpdateTimeStamp;
}

FBasedMovementInfo& ACharacter::GetReplicatedBasedMovement_Mutable()
{
	return ReplicatedBasedMovement;
}

void ACharacter::SetReplicatedBasedMovement(const FBasedMovementInfo& InReplicatedBasedMovement)
{
	ReplicatedBasedMovement = InReplicatedBasedMovement;
}

void ACharacter::SetReplicatedMovementMode(const uint8 InReplicatedMovementMode)
{
	ReplicatedMovementMode = InReplicatedMovementMode; 
}

void ACharacter::SaveRelativeBasedMovement(const FVector& NewRelativeLocation, const FRotator& NewRotation, bool bRelativeRotation)
{
	checkSlow(BasedMovement.HasRelativeLocation());
	BasedMovement.Location = NewRelativeLocation;
	BasedMovement.Rotation = NewRotation;
	BasedMovement.bRelativeRotation = bRelativeRotation;
}

FVector ACharacter::GetGravityDirection() const
{
	FVector GravityDirection = UCharacterMovementComponent::DefaultGravityDirection;
	const UCharacterMovementComponent* const MovementComponent = GetCharacterMovement();
	if (MovementComponent)
	{
		GravityDirection = MovementComponent->GetGravityDirection();
	}

	return GravityDirection;
}

FQuat ACharacter::GetGravityTransform() const
{
	FQuat GravityTransform = FQuat::Identity;
	const UCharacterMovementComponent* const MovementComponent = GetCharacterMovement();
	if (MovementComponent)
	{
		GravityTransform = MovementComponent->GetWorldToGravityTransform();
	}

	return GravityTransform;
}

FVector ACharacter::GetReplicatedGravityDirection() const
{
	return ReplicatedGravityDirection;
}

void ACharacter::SetReplicatedGravityDirection(const FVector& InReplicatedGravityDirection)
{
	ReplicatedGravityDirection = InReplicatedGravityDirection;
}

FVector ACharacter::GetNavAgentLocation() const
{
	FVector AgentLocation = FNavigationSystem::InvalidLocation;

	if (GetCharacterMovement() != nullptr)
	{
		AgentLocation = GetCharacterMovement()->GetActorFeetLocation();
	}

	if (FNavigationSystem::IsValidLocation(AgentLocation) == false && CapsuleComponent != nullptr)
	{
		AgentLocation = GetActorLocation() - FVector(0, 0, CapsuleComponent->GetScaledCapsuleHalfHeight());
	}

	return AgentLocation;
}

void ACharacter::TurnOff()
{
	if (CharacterMovement != nullptr)
	{
		CharacterMovement->StopMovementImmediately();
		CharacterMovement->DisableMovement();
	}

	if (GetNetMode() != NM_DedicatedServer && Mesh != nullptr)
	{
		Mesh->bPauseAnims = true;
		if (Mesh->IsSimulatingPhysics())
		{
			Mesh->bBlendPhysics = true;
			Mesh->KinematicBonesUpdateType = EKinematicBonesUpdateToPhysics::SkipAllBones;
		}
	}

	Super::TurnOff();
}

void ACharacter::Restart()
{
	Super::Restart();

    JumpCurrentCount = 0;
	JumpCurrentCountPreJump = 0;

	bPressedJump = false;
	ResetJumpState();
	UnCrouch(true);

	if (CharacterMovement)
	{
		CharacterMovement->SetDefaultMovementMode();
	}
}

void ACharacter::PawnClientRestart()
{
	if (CharacterMovement != nullptr)
	{
		CharacterMovement->StopMovementImmediately();
		CharacterMovement->ResetPredictionData_Client();
	}

	Super::PawnClientRestart();
}

void ACharacter::PossessedBy(AController* NewController)
{
	Super::PossessedBy(NewController);

	UpdateMeshAllowAutonomousTickPose();
}

void ACharacter::UnPossessed()
{
	Super::UnPossessed();

	if (CharacterMovement)
	{
		CharacterMovement->ResetPredictionData_Client();
		CharacterMovement->ResetPredictionData_Server();
	}

	// We're no longer controlled remotely, resume regular ticking of animations.
	UpdateMeshAllowAutonomousTickPose();
}


void ACharacter::TornOff()
{
	Super::TornOff();

	if (CharacterMovement)
	{
		CharacterMovement->ResetPredictionData_Client();
		CharacterMovement->ResetPredictionData_Server();
	}

	// We're no longer controlled remotely, resume regular ticking of animations.
	UpdateMeshAllowAutonomousTickPose();
}


void ACharacter::NotifyActorBeginOverlap(AActor* OtherActor)
{
	NumActorOverlapEventsCounter++;
	Super::NotifyActorBeginOverlap(OtherActor);
}

void ACharacter::NotifyActorEndOverlap(AActor* OtherActor)
{
	NumActorOverlapEventsCounter++;
	Super::NotifyActorEndOverlap(OtherActor);
}

void ACharacter::BaseChange()
{
	if (CharacterMovement && CharacterMovement->MovementMode != MOVE_None)
	{
		AActor* ActualMovementBase = GetMovementBaseActor(this);
		if ((ActualMovementBase != nullptr) && !ActualMovementBase->CanBeBaseForCharacter(this))
		{
			CharacterMovement->JumpOff(ActualMovementBase);
		}
	}
}

void ACharacter::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);

	float Indent = 0.f;

	static FName NAME_Physics = FName("Physics");
	if (DebugDisplay.IsDisplayOn(NAME_Physics) )
	{
		FIndenter PhysicsIndent(Indent);

		FString BaseString;
		if ( CharacterMovement == nullptr || !BasedMovement.MovementBaseInterfaceData.IsValid() )
		{
			BaseString = "Not Based";
		}
		else
		{
			BaseString = BasedMovement.MovementBaseInterfaceData.GetBodyInstanceOwner()->IsPhysicsObjectWorldGeometry() ? "World Geometry" : BasedMovement.MovementBaseInterfaceData.GetMovementBaseObject()->GetName();
			BaseString = FString::Printf(TEXT("Based On %s"), *BaseString);
		}
		
		FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
		DisplayDebugManager.DrawString(FString::Printf(TEXT("RelativeLoc: %s Rot: %s %s"), *BasedMovement.Location.ToCompactString(), *BasedMovement.Rotation.ToCompactString(), *BaseString), Indent);

		if ( CharacterMovement != nullptr )
		{
			CharacterMovement->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
		}
		const bool Crouched = CharacterMovement && CharacterMovement->IsCrouching();
		FString T = FString::Printf(TEXT("Crouched %i"), Crouched);
		DisplayDebugManager.DrawString(T, Indent);
	}
}

void ACharacter::LaunchCharacter(FVector LaunchVelocity, bool bXYOverride, bool bZOverride)
{
	UE_LOGF(LogCharacter, Verbose, "ACharacter::LaunchCharacter '%ls' (%f,%f,%f)", *GetName(), LaunchVelocity.X, LaunchVelocity.Y, LaunchVelocity.Z);

	if (CharacterMovement)
	{
		FVector FinalVel = LaunchVelocity;
		const FVector Velocity = GetVelocity();

		if (!bXYOverride)
		{
			FinalVel.X += Velocity.X;
			FinalVel.Y += Velocity.Y;
		}
		if (!bZOverride)
		{
			FinalVel.Z += Velocity.Z;
		}

		CharacterMovement->Launch(FinalVel);

		OnLaunched(LaunchVelocity, bXYOverride, bZOverride);
	}
}


void ACharacter::OnMovementModeChanged(EMovementMode PrevMovementMode, uint8 PrevCustomMode)
{
	if (!bPressedJump || !CharacterMovement->IsFalling())
	{
		ResetJumpState();
	}

	// Record jump force start time for proxies. Allows us to expire the jump even if not continually ticking down a timer.
	if (GetProxyIsJumpForceApplied() && CharacterMovement->IsFalling())
	{
		ProxyJumpForceStartedTime = GetWorld()->GetTimeSeconds();
	}

	K2_OnMovementModeChanged(PrevMovementMode, CharacterMovement->MovementMode, PrevCustomMode, CharacterMovement->CustomMovementMode);
	MovementModeChangedDelegate.Broadcast(this, PrevMovementMode, PrevCustomMode);
}


/** Don't process landed notification if updating client position by replaying moves. 
 * Allow event to be called if Pawn was initially falling (before starting to replay moves), 
 * and this is going to cause it to land. . */
bool ACharacter::ShouldNotifyLanded(const FHitResult& Hit)
{
	if (bClientUpdating && !bClientWasFalling)
	{
		return false;
	}

	// Just in case, only allow Landed() to be called once when replaying moves.
	bClientWasFalling = false;
	return true;
}

void ACharacter::Jump()
{
	bPressedJump = true;
	JumpKeyHoldTime = 0.0f;
}

void ACharacter::StopJumping()
{
	bPressedJump = false;
	ResetJumpState();
}

bool ACharacter::CanJumpWhileFalling() const
{
	return false;
}

void ACharacter::CheckJumpInput(float DeltaTime)
{
	JumpCurrentCountPreJump = JumpCurrentCount;

	if (CharacterMovement)
	{
		if (bPressedJump)
		{
			// If this is the first jump and we're already falling, normally we deny that first jump from happening (by skipping right to the second jump, if available)
			// One exception is if our subclass overrides this to allow first jump to still happen while falling under circumstances they care about
			const bool bFirstJump = JumpCurrentCount == 0;
			if (bFirstJump && CharacterMovement->IsFalling() && !CanJumpWhileFalling())
			{
				JumpCurrentCount++;
			}

			const bool bDidJump = CanJump() && (CharacterCVars::UseLegacyDoJump)? CharacterMovement->DoJump(bClientUpdating) : CharacterMovement->DoJump(bClientUpdating, DeltaTime);
			if (bDidJump)
			{
				// Transition from not (actively) jumping to jumping.
				if (!bWasJumping)
				{
					JumpCurrentCount++;
					JumpForceTimeRemaining = GetJumpMaxHoldTime();
					OnJumped();
				}
			}

			bWasJumping = bDidJump;
		}
	}
}


void ACharacter::ClearJumpInput(float DeltaTime)
{
	if (bPressedJump)
	{
		JumpKeyHoldTime += DeltaTime;

		// Don't disable bPressedJump right away if it's still held.
		// Don't modify JumpForceTimeRemaining because a frame of update may be remaining.
		if (JumpKeyHoldTime >= GetJumpMaxHoldTime())
		{
			bPressedJump = false;
		}
	}
	else
	{
		JumpForceTimeRemaining = 0.0f;
		bWasJumping = false;
	}
}

float ACharacter::GetJumpMaxHoldTime() const
{
	return JumpMaxHoldTime;
}

//
// Static variables for networking.
//
static uint8 SavedMovementMode;

void ACharacter::PreNetReceive()
{
	SavedMovementMode = GetReplicatedMovementMode();
	PreNetReceivedGravityDirection = GetReplicatedGravityDirection();
	Super::PreNetReceive();
}

void ACharacter::PostNetReceive()
{
	if (GetLocalRole() == ROLE_SimulatedProxy)
	{
		CharacterMovement->bNetworkGravityDirectionChanged = !PreNetReceivedGravityDirection.Equals(GetReplicatedGravityDirection());
		CharacterMovement->bNetworkMovementModeChanged |= ((SavedMovementMode != GetReplicatedMovementMode()) || (CharacterMovement->PackNetworkMovementMode() != GetReplicatedMovementMode()));
		CharacterMovement->bNetworkUpdateReceived |= CharacterMovement->bNetworkMovementModeChanged || CharacterMovement->bJustTeleported || CharacterMovement->bNetworkGravityDirectionChanged;
	}

	Super::PostNetReceive();
}

void ACharacter::OnRep_ReplicatedBasedMovement()
{	
	// Following the same pattern in AActor::OnRep_ReplicatedMovement() just in case...
	if (!IsReplicatingMovement())
	{
		return;
	}

	if (GetLocalRole() != ROLE_SimulatedProxy)
	{
		return;
	}

	// Skip base updates while playing root motion, it is handled inside of OnRep_RootMotion
	if (IsPlayingNetworkedRootMotionMontage())
	{
		return;
	}

	CharacterMovement->bNetworkUpdateReceived = true;
	FGuardValue_Bitfield(bInBaseReplication, true);

	const FBasedMovementInfo& RepBasedMovement = GetReplicatedBasedMovement();
	FMovementBaseInterfaceData RepMovementBaseInterfaceData = MovementBaseUtility::GetMovementBaseDataFromPhysicsOwner(RepBasedMovement.MovementBaseInterfaceData.PhysicsObjectOwner.Get());
	
	const bool bBaseChanged = (BasedMovement.MovementBaseInterfaceData != RepMovementBaseInterfaceData || BasedMovement.BoneName != RepBasedMovement.BoneName);
	if (bBaseChanged)
	{
		// Even though we will copy the replicated based movement info, we need to use SetBase() to set up tick dependencies and trigger notifications.
		SetBase(&RepMovementBaseInterfaceData, RepBasedMovement.BoneName);
	}

	// Make sure to use the values of relative location/rotation etc from the server.
	BasedMovement = RepBasedMovement;
	BasedMovement.MovementBaseInterfaceData = RepMovementBaseInterfaceData;
	ReplicatedBasedMovement.MovementBaseInterfaceData = RepMovementBaseInterfaceData;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BasedMovement.MovementBase = RepBasedMovement.MovementBase;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	if (BasedMovement.HasRelativeLocation())
	{
		// Update transform relative to movement base
		const FVector OldLocation = GetActorLocation();
		const FQuat OldRotation = GetActorQuat();
		MovementBaseUtility::GetMovementBaseTransform(&BasedMovement.MovementBaseInterfaceData, BasedMovement.BoneName, CharacterMovement->OldBaseLocation, CharacterMovement->OldBaseQuat);
		const FTransform BaseTransform(CharacterMovement->OldBaseQuat, CharacterMovement->OldBaseLocation);
		const FVector NewLocation = BaseTransform.TransformPositionNoScale(BasedMovement.Location);
		FRotator NewRotation;

		if (BasedMovement.HasRelativeRotation())
		{
			// Relative location, relative rotation
			NewRotation = (FRotationMatrix(BasedMovement.Rotation) * FQuatRotationMatrix(CharacterMovement->OldBaseQuat)).Rotator();
			
			if (CharacterMovement->ShouldRemainVertical())
			{
				NewRotation.Pitch = 0.f;
				NewRotation.Roll = 0.f;
			}
		}
		else
		{
			// Relative location, absolute rotation
			NewRotation = BasedMovement.Rotation;
		}

		// When position or base changes, movement mode will need to be updated. This assumes rotation changes don't affect that.
		CharacterMovement->bJustTeleported |= (bBaseChanged || NewLocation != OldLocation);
		CharacterMovement->bNetworkSmoothingComplete = false;
		CharacterMovement->SmoothCorrection(OldLocation, OldRotation, NewLocation, NewRotation.Quaternion());
		OnUpdateSimulatedPosition(OldLocation, OldRotation);
	}
}

void ACharacter::GatherCurrentMovement()
{
	Super::GatherCurrentMovement();

	if (IsReplicatingMovement())
	{
		FRepMovement& MutableRepMovement = GetReplicatedMovement_Mutable();

		if (ShouldReplicateAcceleration()) 
		{
			MutableRepMovement.bRepAcceleration = true;
			MutableRepMovement.Acceleration = CharacterMovement->GetCurrentAcceleration();
		}
		else
		{
			MutableRepMovement.bRepAcceleration = false;
		}
	}
}

void ACharacter::OnRep_ReplicatedMovement()
{
	// Skip standard position correction if we are playing root motion, OnRep_RootMotion will handle it.
	if (!IsPlayingNetworkedRootMotionMontage()) // animation root motion
	{
		if (!CharacterMovement || !CharacterMovement->CurrentRootMotion.HasActiveRootMotionSources()) // root motion sources
		{
			Super::OnRep_ReplicatedMovement();
		}
	}
}

void ACharacter::OnRep_ReplayLastTransformUpdateTimeStamp()
{
	SetReplicatedServerLastTransformUpdateTimeStamp(GetReplayLastTransformUpdateTimeStamp());
}

/** Get FAnimMontageInstance playing RootMotion */
FAnimMontageInstance * ACharacter::GetRootMotionAnimMontageInstance() const
{
	return (Mesh && Mesh->GetAnimInstance()) ? Mesh->GetAnimInstance()->GetRootMotionMontageInstance() : nullptr;
}

FRepRootMotionMontage& ACharacter::GetRepRootMotion_Mutable()
{
	return RepRootMotion;
}
void ACharacter::SetRepRootMotion(const FRepRootMotionMontage& InRepRootMotion)
{
	RepRootMotion = InRepRootMotion;
}

void ACharacter::OnRep_RootMotion()
{
	// Following the same pattern in AActor::OnRep_ReplicatedMovement() just in case...
	if (!IsReplicatingMovement())
	{
		return;
	}

	if (GetLocalRole() == ROLE_SimulatedProxy)
	{

		UE_LOGF(LogRootMotion, Log,  "ACharacter::OnRep_RootMotion");

		// Save received move in queue, we'll try to use it during Tick().
		if( GetRepRootMotion().bIsActive )
		{
			// Add new move
			RootMotionRepMoves.AddZeroed(1);
			FSimulatedRootMotionReplicatedMove& NewMove = RootMotionRepMoves.Last();
			NewMove.RootMotion = GetRepRootMotion();
			NewMove.Time = GetWorld()->GetTimeSeconds();
		}
		else
		{
			// Clear saved moves.
			RootMotionRepMoves.Empty();
		}

		if (CharacterMovement)
		{
			CharacterMovement->bNetworkUpdateReceived = true;
		}
	}
}

void ACharacter::SimulatedRootMotionPositionFixup(float DeltaSeconds)
{
	const FAnimMontageInstance* ClientMontageInstance = GetRootMotionAnimMontageInstance();
	if( ClientMontageInstance && CharacterMovement && Mesh )
	{
		// Find most recent buffered move that we can use.
		const int32 MoveIndex = FindRootMotionRepMove(*ClientMontageInstance);
		if( MoveIndex != INDEX_NONE )
		{
			const FVector OldLocation = GetActorLocation();
			const FQuat OldRotation = GetActorQuat();
			// Move Actor back to position of that buffered move. (server replicated position).
			const FSimulatedRootMotionReplicatedMove& RootMotionRepMove = RootMotionRepMoves[MoveIndex];
			if( RestoreReplicatedMove(RootMotionRepMove) )
			{
				const float ServerPosition = RootMotionRepMove.RootMotion.Position;
				const float ClientPosition = ClientMontageInstance->GetPosition();
				const float DeltaPosition = (ClientPosition - ServerPosition);
				if( FMath::Abs(DeltaPosition) > UE_KINDA_SMALL_NUMBER )
				{
					// Find Root Motion delta move to get back to where we were on the client.
					const FTransform LocalRootMotionTransform = ClientMontageInstance->Montage->ExtractRootMotionFromTrackRange(ServerPosition, ClientPosition, FAnimExtractContext());

					// Simulate Root Motion for delta move.
					if( CharacterMovement )
					{
						const float MontagePlayRate = ClientMontageInstance->GetPlayRate();
						// Guess time it takes for this delta track position, so we can get falling physics accurate.
						if (!FMath::IsNearlyZero(MontagePlayRate))
						{
							const float DeltaTime = DeltaPosition / MontagePlayRate;

							// Even with negative playrate deltatime should be positive.
							check(DeltaTime > 0.f);
							CharacterMovement->SimulateRootMotion(DeltaTime, LocalRootMotionTransform);

							// After movement correction, smooth out error in position if any.
							const FVector NewLocation = GetActorLocation();
							CharacterMovement->bNetworkSmoothingComplete = false;
							CharacterMovement->bJustTeleported |= (OldLocation != NewLocation);
							CharacterMovement->SmoothCorrection(OldLocation, OldRotation, NewLocation, GetActorQuat());
						}
					}
				}
			}

			// Delete this move and any prior one, we don't need them anymore.
			UE_LOGF(LogRootMotion, Log,  "\tClearing old moves (%d)", MoveIndex+1);
			RootMotionRepMoves.RemoveAt(0, MoveIndex+1);
		}
	}
}

int32 ACharacter::FindRootMotionRepMove(const FAnimMontageInstance& ClientMontageInstance) const
{
	int32 FoundIndex = INDEX_NONE;

	// Start with most recent move and go back in time to find a usable move.
	for(int32 MoveIndex=RootMotionRepMoves.Num()-1; MoveIndex>=0; MoveIndex--)
	{
		if( CanUseRootMotionRepMove(RootMotionRepMoves[MoveIndex], ClientMontageInstance) )
		{
			FoundIndex = MoveIndex;
			break;
		}
	}

	UE_LOGF(LogRootMotion, Log,  "\tACharacter::FindRootMotionRepMove FoundIndex: %d, NumSavedMoves: %d", FoundIndex, RootMotionRepMoves.Num());
	return FoundIndex;
}

bool ACharacter::CanUseRootMotionRepMove(const FSimulatedRootMotionReplicatedMove& RootMotionRepMove, const FAnimMontageInstance& ClientMontageInstance) const
{
	// Ignore outdated moves.
	if( GetWorld()->TimeSince(RootMotionRepMove.Time) <= 0.5f )
	{
		// Make sure montage being played matched between client and server.
		UAnimSequenceBase* ClientAnimation = ClientMontageInstance.Montage && ClientMontageInstance.Montage->IsDynamicMontage() ? ClientMontageInstance.Montage->GetFirstAnimReference() : ClientMontageInstance.Montage.Get();
		if (RootMotionRepMove.RootMotion.Animation && RootMotionRepMove.RootMotion.Animation == ClientAnimation)
		{
			UAnimMontage* AnimMontage = ClientMontageInstance.Montage;

			const float ServerPosition = RootMotionRepMove.RootMotion.Position;
			const float ClientPosition = ClientMontageInstance.GetPosition();
			const float DeltaPosition = (ClientPosition - ServerPosition);
			const int32 CurrentSectionIndex = AnimMontage->GetSectionIndexFromPosition(ClientPosition);
			if( CurrentSectionIndex != INDEX_NONE )
			{
				const int32 NextSectionIndex = ClientMontageInstance.GetNextSectionID(CurrentSectionIndex);

				// We can only extract root motion if we are within the same section.
				// It's not trivial to jump through sections in a deterministic manner, but that is luckily not frequent. 
				const bool bSameSections = (AnimMontage->GetSectionIndexFromPosition(ServerPosition) == CurrentSectionIndex);
				// if we are looping and just wrapped over, skip. That's also not easy to handle and not frequent.
				const bool bHasLooped = (NextSectionIndex == CurrentSectionIndex) && (FMath::Abs(DeltaPosition) > (AnimMontage->GetSectionLength(CurrentSectionIndex) / 2.f));
				// Can only simulate forward in time, so we need to make sure server move is not ahead of the client.
				const bool bServerAheadOfClient = ((DeltaPosition * ClientMontageInstance.GetPlayRate()) < 0.f);

				UE_LOGF(LogRootMotion, Log,  "\t\tACharacter::CanUseRootMotionRepMove ServerPosition: %.3f, ClientPosition: %.3f, DeltaPosition: %.3f, bSameSections: %d, bHasLooped: %d, bServerAheadOfClient: %d", 
					ServerPosition, ClientPosition, DeltaPosition, bSameSections, bHasLooped, bServerAheadOfClient);

				return bSameSections && !bHasLooped && !bServerAheadOfClient;
			}
		}
	}
	return false;
}

bool ACharacter::RestoreReplicatedMove(const FSimulatedRootMotionReplicatedMove& RootMotionRepMove)
{
	const FName ServerBaseBoneName = RootMotionRepMove.RootMotion.MovementBaseBoneName;

	FMovementBaseInterfaceData ServerMovementBaseInterfaceData = MovementBaseUtility::GetMovementBaseDataFromPhysicsOwner(RootMotionRepMove.RootMotion.PhysicsObjectOwner);
	
	// Relative Position
	if( RootMotionRepMove.RootMotion.bRelativePosition )
	{
		bool bSuccess = false;
		if( MovementBaseUtility::UseRelativeLocation(&ServerMovementBaseInterfaceData) )
		{
			FVector BaseLocation;
			FQuat BaseRotation;
			MovementBaseUtility::GetMovementBaseTransform(&ServerMovementBaseInterfaceData, ServerBaseBoneName, BaseLocation, BaseRotation);
			const FTransform BaseTransform(BaseRotation, BaseLocation);
			
			const FVector ServerLocation = BaseTransform.TransformPositionNoScale(RootMotionRepMove.RootMotion.Location);
			FRotator ServerRotation;
			if (RootMotionRepMove.RootMotion.bRelativeRotation)
			{
				// Relative rotation
				ServerRotation = (FRotationMatrix(RootMotionRepMove.RootMotion.Rotation) * FQuatRotationTranslationMatrix(BaseRotation, FVector::ZeroVector)).Rotator();
			}
			else
			{
				// Absolute rotation
				ServerRotation = RootMotionRepMove.RootMotion.Rotation;
			}

			SetActorLocationAndRotation(ServerLocation, ServerRotation);
			bSuccess = true;
		}
		// If we received local space position, but can't resolve parent, then move can't be used. :(
		if( !bSuccess )
		{
			return false;
		}
	}
	// Absolute position
	else
	{
		FVector LocalLocation = FRepMovement::RebaseOntoLocalOrigin(RootMotionRepMove.RootMotion.Location, this);
		SetActorLocationAndRotation(LocalLocation, RootMotionRepMove.RootMotion.Rotation);
	}

	CharacterMovement->bJustTeleported = true;
	SetBase(&ServerMovementBaseInterfaceData, ServerBaseBoneName);

	return true;
}

void ACharacter::OnUpdateSimulatedPosition(const FVector& OldLocation, const FQuat& OldRotation)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterOnNetUpdateSimulatedPosition);

	bSimGravityDisabled = false;
	const bool bLocationChanged = (OldLocation != GetActorLocation());
	if (bClientCheckEncroachmentOnNetUpdate)
	{	
		// Only need to check for encroachment when teleported without any velocity.
		// Normal movement pops the character out of geometry anyway, no use doing it before and after (with different rules).
		// Always consider Location as changed if we were spawned this tick as in that case our replicated Location was set as part of spawning, before PreNetReceive()
		if (CharacterMovement->Velocity.IsZero() && (bLocationChanged || CreationTime == GetWorld()->TimeSeconds))
		{
			if (GetWorld()->EncroachingBlockingGeometry(this, GetActorLocation(), GetActorRotation()))
			{
				bSimGravityDisabled = true;
			}
		}
	}
	CharacterMovement->bJustTeleported |= bLocationChanged;
	CharacterMovement->bNetworkUpdateReceived = true;
}

void ACharacter::PostNetReceiveLocationAndRotation()
{
	if(GetLocalRole() == ROLE_SimulatedProxy)
	{
		const FBasedMovementInfo& RepBasedMovement = GetReplicatedBasedMovement();
		FMovementBaseInterfaceData RepMovementBaseInterfaceData = MovementBaseUtility::GetMovementBaseDataFromPhysicsOwner(RepBasedMovement.MovementBaseInterfaceData.PhysicsObjectOwner.Get());
		
		// Don't change transform if using relative position (it should be nearly the same anyway, or base may be slightly out of sync)
		if (!MovementBaseUtility::UseRelativeLocation(&RepMovementBaseInterfaceData))
		{
			const FRepMovement& ConstRepMovement = GetReplicatedMovement();
			const FVector OldLocation = GetActorLocation();
			const FVector NewLocation = FRepMovement::RebaseOntoLocalOrigin(ConstRepMovement.Location, this);
			const FQuat OldRotation = GetActorQuat();

			CharacterMovement->bNetworkSmoothingComplete = false;
			CharacterMovement->bJustTeleported |= (OldLocation != NewLocation);
			CharacterMovement->SmoothCorrection(OldLocation, OldRotation, NewLocation, ConstRepMovement.Rotation.Quaternion());
			OnUpdateSimulatedPosition(OldLocation, OldRotation);
		}
		CharacterMovement->bNetworkUpdateReceived = true;
	}
}

void ACharacter::PreReplication( IRepChangedPropertyTracker & ChangedPropertyTracker )
{
	Super::PreReplication( ChangedPropertyTracker );

	if (IsReplicatingMovement() && (CharacterMovement->CurrentRootMotion.HasActiveRootMotionSources() || IsPlayingNetworkedRootMotionMontage()))
	{
		const FAnimMontageInstance* RootMotionMontageInstance = GetRootMotionAnimMontageInstance();

		FRepRootMotionMontage& RepRootMotionRef = GetRepRootMotion_Mutable();
		RepRootMotionRef.bIsActive = true;
		// Is position stored in local space?
		RepRootMotionRef.bRelativePosition = BasedMovement.HasRelativeLocation();
		RepRootMotionRef.bRelativeRotation = BasedMovement.HasRelativeRotation();
		RepRootMotionRef.Location			= RepRootMotionRef.bRelativePosition ? BasedMovement.Location : FRepMovement::RebaseOntoZeroOrigin(GetActorLocation(), GetWorld()->OriginLocation);
		RepRootMotionRef.Rotation			= RepRootMotionRef.bRelativeRotation ? BasedMovement.Rotation : GetActorRotation();
		RepRootMotionRef.PhysicsObjectOwner = BasedMovement.MovementBaseInterfaceData.PhysicsObjectOwner.Get();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RepRootMotionRef.MovementBase = BasedMovement.MovementBase;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		RepRootMotionRef.MovementBaseBoneName = BasedMovement.BoneName;
		if (RootMotionMontageInstance)
		{
			RepRootMotionRef.Animation = RootMotionMontageInstance->Montage->IsDynamicMontage() ? RootMotionMontageInstance->Montage->GetFirstAnimReference() : RootMotionMontageInstance->Montage;
			RepRootMotionRef.Position			= RootMotionMontageInstance->GetPosition();
		}
		else
		{
			RepRootMotionRef.Animation = nullptr;
		}

		RepRootMotionRef.AuthoritativeRootMotion = CharacterMovement->CurrentRootMotion;
		RepRootMotionRef.Acceleration = CharacterMovement->GetCurrentAcceleration();
		RepRootMotionRef.LinearVelocity = CharacterMovement->Velocity;

		DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(ACharacter, RepRootMotion, true);
	}
	else
	{
		const bool bWasRootMotionPreviouslyActive = GetRepRootMotion().bIsActive;

		FRepRootMotionMontage& RepRootMotionRef = GetRepRootMotion_Mutable();
		RepRootMotionRef.Clear();
		
		// Replicate RepRootMotion one last time when root motion ends, so that clients see the change.
		// Then deactivate subsequent property comparisons and replication updates until root motion starts again.
		DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(ACharacter, RepRootMotion, bWasRootMotionPreviouslyActive);
	}

	SetProxyIsJumpForceApplied(JumpForceTimeRemaining > 0.0f);
	SetProxyCharacterMovementActive(CharacterMovement->IsActive());
	SetReplicatedMovementMode(CharacterMovement->PackNetworkMovementMode());
	SetReplicatedGravityDirection(CharacterMovement->GetGravityDirection());

	if(IsReplicatingMovement())
	{
		FBasedMovementInfo& ReplicatedBasedMovementRef = GetReplicatedBasedMovement_Mutable();
		ReplicatedBasedMovementRef = BasedMovement;

		// Optimization: only update and replicate these values if they are actually going to be used.
		if (BasedMovement.HasRelativeLocation())
		{
			// When velocity becomes zero, force replication so the position is updated to match the server (it may have moved due to simulation on the client).
			ReplicatedBasedMovementRef.bServerHasVelocity = !CharacterMovement->Velocity.IsZero();

			// Make sure absolute rotations are updated in case rotation occurred after the base info was saved.
			if (!BasedMovement.HasRelativeRotation())
			{
				ReplicatedBasedMovementRef.Rotation = GetActorRotation();
			}
		}
	}

	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(ACharacter, ReplicatedBasedMovement, IsReplicatingMovement());

	// Save bandwidth by not replicating this value unless it is necessary, since it changes every update.
	if ((CharacterMovement->NetworkSmoothingMode == ENetworkSmoothingMode::Linear) || CharacterMovement->bNetworkAlwaysReplicateTransformUpdateTimestamp)
	{
		SetReplicatedServerLastTransformUpdateTimeStamp(CharacterMovement->GetServerLastTransformUpdateTimeStamp());
	}
	else
	{
		SetReplicatedServerLastTransformUpdateTimeStamp(0.f);
	}
}

void ACharacter::GetReplicatedCustomConditionState(FCustomPropertyConditionState& OutActiveState) const
{
	Super::GetReplicatedCustomConditionState(OutActiveState);

	DOREPCUSTOMCONDITION_ACTIVE_FAST(ACharacter, RepRootMotion, CharacterMovement->CurrentRootMotion.HasActiveRootMotionSources() || IsPlayingNetworkedRootMotionMontage());
}

void ACharacter::PreReplicationForReplay(IRepChangedPropertyTracker& ChangedPropertyTracker)
{
	Super::PreReplicationForReplay(ChangedPropertyTracker);

	const UWorld* World = GetWorld();
	if (World)
	{
		// On client replays, our view pitch will be set to 0 as by default we do not replicate
		// pitch for owners, just for simulated. So instead push our rotation into the sampler
		if (World->IsRecordingClientReplay() && GetController() != nullptr && GetLocalRole() == ROLE_AutonomousProxy && GetNetMode() == NM_Client)
		{
			SetRemoteViewPitch(GetController()->GetControlRotation().Pitch);
		}

		SetReplayLastTransformUpdateTimeStamp(World->GetTimeSeconds());
	}
}

void ACharacter::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );

	DISABLE_REPLICATED_PROPERTY(ACharacter, JumpMaxHoldTime);
	DISABLE_REPLICATED_PROPERTY(ACharacter, JumpMaxCount);

	DOREPLIFETIME_CONDITION( ACharacter, RepRootMotion,						COND_SimulatedOnly );
	DOREPLIFETIME_CONDITION( ACharacter, ReplicatedBasedMovement,			COND_SimulatedOnly );
	DOREPLIFETIME_CONDITION( ACharacter, ReplicatedServerLastTransformUpdateTimeStamp, COND_SimulatedOnlyNoReplay );
	DOREPLIFETIME_CONDITION( ACharacter, ReplicatedMovementMode,			COND_SimulatedOnly );
	DOREPLIFETIME_CONDITION( ACharacter, bIsCrouched,						COND_SimulatedOnly );
	DOREPLIFETIME_CONDITION( ACharacter, bProxyIsCharacterMovementActive,	COND_SimulatedOnly );
	DOREPLIFETIME_CONDITION( ACharacter, bProxyIsJumpForceApplied,			COND_SimulatedOnly );
	DOREPLIFETIME_CONDITION( ACharacter, AnimRootMotionTranslationScale,	COND_SimulatedOnly );
	DOREPLIFETIME_CONDITION( ACharacter, ReplicatedGravityDirection,		COND_SimulatedOnly );
	DOREPLIFETIME_CONDITION( ACharacter, ReplayLastTransformUpdateTimeStamp, COND_ReplayOnly );
}

bool ACharacter::IsPlayingRootMotion() const
{
	if (Mesh)
	{
		return Mesh->IsPlayingRootMotion();
	}
	return false;
}

bool ACharacter::HasAnyRootMotion() const
{
	return CharacterMovement ? CharacterMovement->HasRootMotionSources() : false;
}

bool ACharacter::IsPlayingNetworkedRootMotionMontage() const
{
	if (Mesh)
	{
		return Mesh->IsPlayingNetworkedRootMotionMontage();
	}
	return false;
}

void ACharacter::SetAnimRootMotionTranslationScale(float InAnimRootMotionTranslationScale)
{
	AnimRootMotionTranslationScale = InAnimRootMotionTranslationScale;
}

float ACharacter::GetAnimRootMotionTranslationScale() const
{
	return AnimRootMotionTranslationScale;
}

float ACharacter::PlayAnimMontage(class UAnimMontage* AnimMontage, float InPlayRate, FName StartSectionName)
{
	UAnimInstance * AnimInstance = (Mesh)? Mesh->GetAnimInstance() : nullptr; 
	if( AnimMontage && AnimInstance )
	{
		float const Duration = AnimInstance->Montage_Play(AnimMontage, InPlayRate);

		if (Duration > 0.f)
		{
			// Start at a given Section.
			if( StartSectionName != NAME_None )
			{
				AnimInstance->Montage_JumpToSection(StartSectionName, AnimMontage);
			}

			return Duration;
		}
	}	

	return 0.f;
}

void ACharacter::StopAnimMontage(class UAnimMontage* AnimMontage)
{
	UAnimInstance * AnimInstance = (Mesh)? Mesh->GetAnimInstance() : nullptr; 
	UAnimMontage * MontageToStop = (AnimMontage)? AnimMontage : GetCurrentMontage();
	bool bShouldStopMontage =  AnimInstance && MontageToStop && !AnimInstance->Montage_GetIsStopped(MontageToStop);

	if ( bShouldStopMontage )
	{
		AnimInstance->Montage_Stop(MontageToStop->BlendOut.GetBlendTime(), MontageToStop);
	}
}

class UAnimMontage * ACharacter::GetCurrentMontage() const
{
	UAnimInstance * AnimInstance = (Mesh)? Mesh->GetAnimInstance() : nullptr; 
	if ( AnimInstance )
	{
		return AnimInstance->GetCurrentActiveMontage();
	}

	return nullptr;
}

void ACharacter::ClientCheatWalk_Implementation()
{
#if !UE_BUILD_SHIPPING
	SetActorEnableCollision(true);
	if (CharacterMovement)
	{
		CharacterMovement->bCheatFlying = false;
		CharacterMovement->SetMovementMode(MOVE_Falling);
	}
#endif
}

void ACharacter::ClientCheatFly_Implementation()
{
#if !UE_BUILD_SHIPPING
	SetActorEnableCollision(true);
	if (CharacterMovement)
	{
		CharacterMovement->bCheatFlying = true;
		CharacterMovement->SetMovementMode(MOVE_Flying);
	}
#endif
}

void ACharacter::ClientCheatGhost_Implementation()
{
#if !UE_BUILD_SHIPPING
	SetActorEnableCollision(false);
	if (CharacterMovement)
	{
		CharacterMovement->bCheatFlying = true;
		CharacterMovement->SetMovementMode(MOVE_Flying);
	}
#endif
}

void ACharacter::RootMotionDebugClientPrintOnScreen_Implementation(const FString& InString)
{
#if ROOT_MOTION_DEBUG
	RootMotionSourceDebug::PrintOnScreenServerMsg(InString);
#endif
}


// ServerMovePacked
void ACharacter::ServerMovePacked_Implementation(const FCharacterServerMovePackedBits& PackedBits)
{
	GetCharacterMovement()->ServerMovePacked_ServerReceive(PackedBits);
}

bool ACharacter::ServerMovePacked_Validate(const FCharacterServerMovePackedBits& PackedBits)
{
	// Can't really validate the bit stream without unpacking, and that is done in ServerMovePacked_ServerReceive() and can be rejected after unpacking.
	return true;
}

// ServerMove
void ACharacter::ServerMove_Implementation(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	GetCharacterMovement()->ServerMove_Implementation(TimeStamp, InAccel, ClientLoc, CompressedMoveFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
}

bool ACharacter::ServerMove_Validate(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	return GetCharacterMovement()->ServerMove_Validate(TimeStamp, InAccel, ClientLoc, CompressedMoveFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
}

// ServerMoveNoBase
void ACharacter::ServerMoveNoBase_Implementation(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode)
{
	GetCharacterMovement()->ServerMove_Implementation(TimeStamp, InAccel, ClientLoc, CompressedMoveFlags, ClientRoll, View, /*ClientMovementBase=*/ nullptr, /*ClientBaseBoneName=*/ NAME_None, ClientMovementMode);
}

bool ACharacter::ServerMoveNoBase_Validate(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode)
{
	return GetCharacterMovement()->ServerMove_Validate(TimeStamp, InAccel, ClientLoc, CompressedMoveFlags, ClientRoll, View, /*ClientMovementBase=*/ nullptr, /*ClientBaseBoneName=*/ NAME_None, ClientMovementMode);
}

// ServerMoveDual
void ACharacter::ServerMoveDual_Implementation(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	GetCharacterMovement()->ServerMoveDual_Implementation(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
}

bool ACharacter::ServerMoveDual_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	return GetCharacterMovement()->ServerMoveDual_Validate(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
}

// ServerMoveDualNoBase
void ACharacter::ServerMoveDualNoBase_Implementation(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode)
{
	GetCharacterMovement()->ServerMoveDual_Implementation(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, /*ClientMovementBase=*/ nullptr, /*ClientBaseBoneName=*/ NAME_None, ClientMovementMode);
}

bool ACharacter::ServerMoveDualNoBase_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode)
{
	return GetCharacterMovement()->ServerMoveDual_Validate(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, /*ClientMovementBase=*/ nullptr, /*ClientBaseBoneName=*/ NAME_None, ClientMovementMode);
}

// ServerMoveDualHybridRootMotion
void ACharacter::ServerMoveDualHybridRootMotion_Implementation(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	GetCharacterMovement()->ServerMoveDualHybridRootMotion_Implementation(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
}

bool ACharacter::ServerMoveDualHybridRootMotion_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	return GetCharacterMovement()->ServerMoveDualHybridRootMotion_Validate(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
}

// ServerMoveOld
void ACharacter::ServerMoveOld_Implementation(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags)
{
	GetCharacterMovement()->ServerMoveOld_Implementation(OldTimeStamp, OldAccel, OldMoveFlags);
}

bool ACharacter::ServerMoveOld_Validate(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags)
{
	return GetCharacterMovement()->ServerMoveOld_Validate(OldTimeStamp, OldAccel, OldMoveFlags);
}

// ClientMoveResponsePacked
void ACharacter::ClientMoveResponsePacked_Implementation(const FCharacterMoveResponsePackedBits& PackedBits)
{
	GetCharacterMovement()->MoveResponsePacked_ClientReceive(PackedBits);
}

bool ACharacter::ClientMoveResponsePacked_Validate(const FCharacterMoveResponsePackedBits& PackedBits)
{
	// Can't really validate the bit stream without unpacking, and that is done in MoveResponsePacked_ClientReceive() and can be rejected after unpacking.
	return true;
}

// ClientAckGoodMove
void ACharacter::ClientAckGoodMove_Implementation(float TimeStamp)
{
	GetCharacterMovement()->ClientAckGoodMove_Implementation(TimeStamp);
}

// ClientAdjustPosition
void ACharacter::ClientAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, FVector NewVel, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	FMovementBaseInterfaceData NewMovementBaseData(NewBase);
	GetCharacterMovement()->ClientAdjustPosition_Implementation(TimeStamp, NewLoc, NewVel, &NewMovementBaseData, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

// ClientVeryShortAdjustPosition
void ACharacter::ClientVeryShortAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	FMovementBaseInterfaceData NewMovementBaseData(NewBase);
	GetCharacterMovement()->ClientVeryShortAdjustPosition_Implementation(TimeStamp, NewLoc, &NewMovementBaseData, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

// ClientAdjustRootMotionPosition
void ACharacter::ClientAdjustRootMotionPosition_Implementation(float TimeStamp, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	FMovementBaseInterfaceData ServerMovementBaseData(ServerBase);
	GetCharacterMovement()->ClientAdjustRootMotionPosition_Implementation(TimeStamp, ServerMontageTrackPosition, ServerLoc, ServerRotation, ServerVelZ, &ServerMovementBaseData, ServerBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

// ClientAdjustRootMotionSourcePosition
void ACharacter::ClientAdjustRootMotionSourcePosition_Implementation(float TimeStamp, FRootMotionSourceGroup ServerRootMotion, bool bHasAnimRootMotion, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	FMovementBaseInterfaceData ServerMovementBaseData(ServerBase);
	GetCharacterMovement()->ClientAdjustRootMotionSourcePosition_Implementation(TimeStamp, ServerRootMotion, bHasAnimRootMotion, ServerMontageTrackPosition, ServerLoc, ServerRotation, ServerVelZ, &ServerMovementBaseData, ServerBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

void ACharacter::FillAsyncInput(FCharacterAsyncInput& Input) const
{
	Input.JumpMaxHoldTime = GetJumpMaxHoldTime();
	Input.JumpMaxCount = JumpMaxCount;
	Input.LocalRole = ENetRole::ROLE_Authority;//CharacterOwner->GetLocalRole(); Override as we aren't currently replicating to server. TODO NetRole
	Input.RemoteRole = GetRemoteRole();
	Input.bIsLocallyControlled = true;// CharacterOwner->IsLocallyControlled(); TODO NetRole
	Input.bIsPlayingNetworkedRootMontage = IsPlayingNetworkedRootMotionMontage();
	Input.bUseControllerRotationPitch = bUseControllerRotationPitch;
	Input.bUseControllerRotationYaw = bUseControllerRotationYaw;
	Input.bUseControllerRotationRoll = bUseControllerRotationRoll;
	Input.ControllerDesiredRotation = GetController()->GetDesiredRotation();
}

void ACharacter::InitializeAsyncOutput(FCharacterAsyncOutput& Output) const
{
	Output.Rotation = GetActorRotation();
	Output.JumpCurrentCountPreJump = JumpCurrentCountPreJump;
	Output.JumpCurrentCount = JumpCurrentCount;
	Output.JumpForceTimeRemaining = JumpForceTimeRemaining;
	Output.bWasJumping = bWasJumping;
	Output.bPressedJump = bPressedJump;
	Output.JumpKeyHoldTime = JumpKeyHoldTime;
	Output.bClearJumpInput = false;
}

void ACharacter::ApplyAsyncOutput(const FCharacterAsyncOutput& Output)
{
	JumpCurrentCountPreJump = Output.JumpCurrentCountPreJump;
	JumpCurrentCount = Output.JumpCurrentCount;
	JumpForceTimeRemaining = Output.JumpForceTimeRemaining;
	bWasJumping = Output.bWasJumping;
	JumpKeyHoldTime = Output.JumpKeyHoldTime;

	if (Output.bClearJumpInput)
	{
		bPressedJump = false;
	}
}

void ACharacter::UpdateMeshAllowAutonomousTickPose()
{
	if (Mesh)
	{
		// If we are controlled remotely and CharacterMovement is active, set animation timing to be driven by client's network updates. So timing and events remain in sync.
		const bool bCharacterMovementWillTickPose = IsReplicatingMovement() && (CharacterMovement && CharacterMovement->IsActive());
		const bool bRemotelyControlled = (GetRemoteRole() == ROLE_AutonomousProxy && GetNetConnection() != nullptr);

		Mesh->bOnlyAllowAutonomousTickPose = bCharacterMovementWillTickPose && bRemotelyControlled;
	}
}

UAnimMontage* FRepRootMotionMontage::GetAnimMontage() const
{
	return Cast<UAnimMontage>(Animation);
}