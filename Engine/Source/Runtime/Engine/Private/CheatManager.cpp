// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/CheatManager.h"
#include "Engine/ServerStatReplicator.h"
#include "Misc/OutputDeviceFile.h"
#include "GameFramework/CheatManagerDefines.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/DamageType.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LocalPlayer.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/GameModeBase.h"
#include "EngineUtils.h"
#include "Net/OnlineEngineInterface.h"
#include "VisualLogger/VisualLogger.h"
#include "GameFramework/Character.h"
#include "Engine/Console.h"
#include "Engine/DebugCameraController.h"
#include "Components/CapsuleComponent.h"
#include "Components/BrushComponent.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/InputSettings.h"
#include "Misc/CoreDelegates.h"
#include "Engine/NetConnection.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Engine/DamageEvents.h"
#include "DebugCameraManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CheatManager)

DEFINE_LOG_CATEGORY_STATIC(LogCheatManager, Log, All);

#define LOCTEXT_NAMESPACE "CheatManager"	

bool UCheatManager::bDebugCapsuleSweepPawn = false;

FOnCheatManagerCreated UCheatManager::OnCheatManagerCreatedDelegate;

UWorld* UCheatManagerExtension::GetWorld() const
{
	if (IsTemplate())
	{
		return nullptr;
	}

	return GetOuterUCheatManager()->GetWorld();
}

APlayerController* UCheatManagerExtension::GetPlayerController() const
{
	return GetOuterUCheatManager()->GetPlayerController();
}

void UCheatManagerExtension::AddedToCheatManager_Implementation()
{
}

void UCheatManagerExtension::RemovedFromCheatManager_Implementation()
{
}

void UCheatManagerExtension::DoExtensionSpecificBugItLog(FOutputDevice& OutputFile)
{
}

UCheatManager::UCheatManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bToggleAILogging(false)
{
	DebugCameraControllerClass = ADebugCameraController::StaticClass();
	DebugCapsuleHalfHeight = 23.0f;
	DebugCapsuleRadius = 21.0f;
	DebugTraceDistance = 10000.0f;
	DebugTraceDrawNormalLength = 30.0f;
	DebugTraceChannel = ECC_Pawn;
	bDebugCapsuleTraceComplex = false;
}

void UCheatManager::OnPlayerEndPlayed(AActor* Player, EEndPlayReason::Type EndPlayReason)
{
	for (UCheatManagerExtension* CheatExtension : CheatManagerExtensions)
	{
		CheatExtension->RemovedFromCheatManager();
	}
	CheatManagerExtensions.Empty();
}

bool UCheatManager::ProcessConsoleExec(const TCHAR* Cmd, FOutputDevice& Ar, UObject* Executor)
{
#if UE_WITH_CHEAT_MANAGER
	// If on the client and calling a cheat function marked as BlueprintAuthorityOnly, automatically route it through the ServerExec() RPC to the server
	APlayerController* MyPC = GetOuterAPlayerController();
	if (MyPC->GetLocalRole() != ROLE_Authority)
	{
		const TCHAR* TestCmd = Cmd;

		FString FunctionNameStr;
		if (FParse::Token(TestCmd, FunctionNameStr, true))
		{
			const FName FunctionName = FName(*FunctionNameStr, FNAME_Find);

			if (FunctionName != NAME_None)
			{
				// Check first in this class
				UFunction* Function = FindFunction(FunctionName);

				// Failing that, check in each of the child cheat managers for a function by this name
				if (Function == nullptr)
				{
					for (UObject* CheatObject : CheatManagerExtensions)
					{
						Function = CheatObject ? CheatObject->FindFunction(FunctionName) : nullptr;
						if (Function != nullptr)
						{
							break;
						}
					}
				}

				if ((Function != nullptr) && Function->HasAnyFunctionFlags(FUNC_BlueprintAuthorityOnly))
				{
					if(ensureMsgf(GAllowActorScriptExecutionInEditor == false, TEXT("GAllowActorScriptExecutionInEditor must be false when executing commands.")))
					{
						MyPC->ServerExec(Cmd);
					}
					return true;
				}
			}
		}
	}
#endif

	for (UObject* CheatObject : CheatManagerExtensions)
	{
		if ((CheatObject != nullptr) && CheatObject->ProcessConsoleExec(Cmd, Ar, Executor))
		{
			return true;
		}
	}

	return Super::ProcessConsoleExec(Cmd, Ar, Executor);
}

void UCheatManager::FreezeFrame(float delay)
{
	if (UWorld* World = GetWorld())
	{
		if (AGameModeBase* GameMode = World->GetAuthGameMode())
		{
			check(GetOuterAPlayerController() != NULL);

			FCanUnpause DefaultCanUnpause;
			DefaultCanUnpause.BindUObject(GetOuterAPlayerController(), &APlayerController::DefaultCanUnpause);
			GameMode->SetPause(GetOuterAPlayerController(), DefaultCanUnpause);
			World->PauseDelay = World->TimeSeconds + delay;
		}
	}
}

void UCheatManager::Teleport()
{	
	FVector	ViewLocation;
	FRotator ViewRotation;
	check(GetOuterAPlayerController() != NULL);
	GetOuterAPlayerController()->GetPlayerViewPoint( ViewLocation, ViewRotation );

	FHitResult Hit;

	APawn* AssociatedPawn = GetOuterAPlayerController()->GetPawn();
	FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(TeleportTrace), true, AssociatedPawn);

	bool bHit = GetWorld()->LineTraceSingleByChannel(Hit, ViewLocation, ViewLocation + 1000000.f * ViewRotation.Vector(), ECC_Pawn, TraceParams);
	if ( bHit )
	{
		Hit.Location += Hit.Normal * 4.0f;
	}

	if (AssociatedPawn != NULL)
	{
		AssociatedPawn->TeleportTo( Hit.Location, AssociatedPawn->GetActorRotation() );
	}
	else
	{
		ADebugCameraController* const DCC = Cast<ADebugCameraController>(GetOuter());
		if ((DCC != NULL) && (DCC->OriginalControllerRef != NULL))
		{
			APawn* OriginalControllerPawn = DCC->OriginalControllerRef->GetPawn();
			if (OriginalControllerPawn != NULL)
			{
				OriginalControllerPawn->TeleportTo(Hit.Location, OriginalControllerPawn->GetActorRotation());
			}
		}
	}
}

void UCheatManager::ChangeSize( float F )
{
	APawn* Pawn = GetOuterAPlayerController()->GetPawn();
	
	// Note: only works on characters
	ACharacter *Character =  Cast<ACharacter>(Pawn);
	if (Character)
	{
		ACharacter* DefaultCharacter = Character->GetClass()->GetDefaultObject<ACharacter>();
		Character->GetCapsuleComponent()->SetCapsuleSize(DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleRadius() * F, DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight() * F);

		if (Character->GetMesh())
		{
			Character->GetMesh()->SetRelativeScale3D(FVector(F));
		}
		Character->TeleportTo(Character->GetActorLocation(), Character->GetActorRotation());
	}
}


void UCheatManager::Fly()
{
	APawn* Pawn = GetOuterAPlayerController()->GetPawn();
	if (Pawn != NULL)
	{
		GetOuterAPlayerController()->ClientMessage(TEXT("You feel much lighter"));
		
		ACharacter* Character =  Cast<ACharacter>(Pawn);
		if (Character)
		{
			Character->ClientCheatFly();
			if (!Character->IsLocallyControlled())
			{
				Character->ClientCheatFly_Implementation();
			}
		}
	}
}

void UCheatManager::Walk()
{
	APawn* Pawn = GetOuterAPlayerController()->GetPawn();
	if (Pawn != NULL)
	{
		ACharacter* Character =  Cast<ACharacter>(Pawn);
		if (Character)
		{
			Character->ClientCheatWalk();
			if (!Character->IsLocallyControlled())
			{
				Character->ClientCheatWalk_Implementation();
			}
		}
	}
}

void UCheatManager::Ghost()
{
	APawn* Pawn = GetOuterAPlayerController()->GetPawn();
	if (Pawn != NULL)
	{
		GetOuterAPlayerController()->ClientMessage(TEXT("You feel ethereal"));
		
		ACharacter* Character =  Cast<ACharacter>(Pawn);
		if (Character)
		{
			Character->ClientCheatGhost();
			if (!Character->IsLocallyControlled())
			{
				Character->ClientCheatGhost_Implementation();
			}
		}
	}	
}

void UCheatManager::God()
{
	APawn* Pawn = GetOuterAPlayerController()->GetPawn();
	if (Pawn != nullptr)
	{
		if (Pawn->CanBeDamaged())
		{
			Pawn->SetCanBeDamaged(false);
			GetOuterAPlayerController()->ClientMessage(TEXT("God mode on"));
		}
		else
		{
			Pawn->SetCanBeDamaged(true);
			GetOuterAPlayerController()->ClientMessage(TEXT("God Mode off"));
		}
	}
	else
	{
		GetOuterAPlayerController()->ClientMessage(TEXT("No APawn* possessed"));
	}
}

void UCheatManager::Slomo(float NewTimeDilation)
{
	GetOuterAPlayerController()->GetWorldSettings()->SetTimeDilation(NewTimeDilation);
}

void UCheatManager::DamageTarget(float DamageAmount)
{
	APlayerController* const MyPC = GetOuterAPlayerController();
	FHitResult Hit;
    AActor* TargetActor = GetTarget(MyPC, Hit);
    if (TargetActor)
	{
		FVector ActorForward, ActorSide, ActorUp;
        FQuatRotationMatrix(TargetActor->GetActorQuat()).GetScaledAxes(ActorForward, ActorSide, ActorUp);
        
		FPointDamageEvent DamageEvent(DamageAmount, Hit, -ActorForward, UDamageType::StaticClass());
        TargetActor->TakeDamage(DamageAmount, DamageEvent, MyPC, MyPC->GetPawn());
	}
}

void UCheatManager::DestroyTarget()
{
	APlayerController* const MyPC = GetOuterAPlayerController();
	FHitResult Hit;
    AActor* TargetActor = GetTarget(MyPC, Hit);
    if (TargetActor)
	{
        APawn* Pawn = Cast<APawn>(TargetActor);
		if (Pawn != NULL)
		{
			if ((Pawn->GetController() != NULL) && (Cast<APlayerController>(Pawn->GetController()) == NULL))
			{
				// Destroy any associated controller as long as it's not a player controller.
				Pawn->GetController()->Destroy();
			}
		}
        
        TargetActor->Destroy();
	}
}

void UCheatManager::DestroyAll(TSubclassOf<AActor> aClass)
{
	for (TActorIterator<AActor> It(GetWorld(),aClass); It; ++It)
	{
		AActor* A = *It;
		if (IsValidChecked(A))
		{
			APawn* Pawn = Cast<APawn>(A);
			if (Pawn != NULL)
			{
				if ((Pawn->GetController() != NULL) && (Cast<APlayerController>(Pawn->GetController()) == NULL))
				{
					// Destroy any associated controller as long as it's not a player controller.
					Pawn->GetController()->Destroy();
				}
			}
			A->Destroy();
		}
	}
}

void UCheatManager::DestroyAllPawnsExceptTarget()
{
	APlayerController* const MyPC = GetOuterAPlayerController();
	FHitResult Hit;
    APawn* HitPawnTarget = Cast<APawn>(GetTarget(MyPC, Hit));
	// if we have a pawn target, destroy all other non-players
	if (HitPawnTarget)
	{
		for (TActorIterator<APawn> It(GetWorld(), APawn::StaticClass()); It; ++It)
		{
			APawn* Pawn = *It;
			if (IsValidChecked(Pawn))
			{
				if ((Pawn != HitPawnTarget) && Cast<APlayerController>(Pawn->GetController()) == NULL)
				{
					if (Pawn->GetController() != NULL)
					{
						Pawn->GetController()->Destroy();
					}
					Pawn->Destroy();
				}
			}
		}
	}
}

void UCheatManager::DestroyPawns(TSubclassOf<APawn> aClass)
{
	if ( aClass == NULL )
	{
		 aClass = APawn::StaticClass();
	}
	for (TActorIterator<APawn> It(GetWorld(), aClass); It; ++It)
	{
		APawn* Pawn = *It;
		if ( Cast<APlayerController>(Pawn->GetController()) == NULL )
		{
			if ( Pawn->GetController() != NULL )
			{
				Pawn->GetController()->Destroy();
			}
			Pawn->Destroy();
		}
	}
	
}

void UCheatManager::Summon( const FString& ClassName )
{
	UE_LOGF(LogCheatManager, Log, "Fabricate %ls", *ClassName );

	bool bIsValidClassName = true;
	FString FailureReason;
	if ( ClassName.Contains(TEXT(" ")) )
	{
		FailureReason = FString::Printf(TEXT("ClassName contains a space."));
		bIsValidClassName = false;
	}
	else if ( !FPackageName::IsShortPackageName(ClassName) )
	{
		if ( ClassName.Contains(TEXT(".")) )
		{
			FString PackageName;
			FString ObjectName;
			ClassName.Split(TEXT("."), &PackageName, &ObjectName);

			const bool bIncludeReadOnlyRoots = true;
			FText Reason;
			if ( !FPackageName::IsValidLongPackageName(PackageName, bIncludeReadOnlyRoots, &Reason) )
			{
				FailureReason = Reason.ToString();
				bIsValidClassName = false;
			}
		}
		else
		{
			FailureReason = TEXT("Class names with a path must contain a dot. (i.e /Script/Engine.StaticMeshActor)");
			bIsValidClassName = false;
		}
	}

	bool bSpawnedActor = false;
	if ( bIsValidClassName )
	{
		UClass* NewClass = UClass::TryFindTypeSlow<UClass>(ClassName);

		if( NewClass )
		{
			if ( NewClass->IsChildOf(AActor::StaticClass()) )
			{
				APlayerController* const MyPlayerController = GetOuterAPlayerController();
				if(MyPlayerController)
				{
					FRotator const SpawnRot = MyPlayerController->GetControlRotation();
					FVector SpawnLoc = MyPlayerController->GetFocalLocation();

					SpawnLoc += 72.f * SpawnRot.Vector() + FVector(0.f, 0.f, 1.f) * 15.f;
					FActorSpawnParameters SpawnInfo;
					SpawnInfo.Instigator = MyPlayerController->GetInstigator();
					AActor* Actor = GetWorld()->SpawnActor(NewClass, &SpawnLoc, &SpawnRot, SpawnInfo );
					if ( Actor )
					{
						bSpawnedActor = true;
					}
					else
					{
						FailureReason = TEXT("SpawnActor failed.");
						bSpawnedActor = false;
					}
				}
			}
			else
			{
				FailureReason = TEXT("Class is not derived from Actor.");
				bSpawnedActor = false;
			}
		}
		else
		{
			FailureReason = TEXT("Failed to find class.");
			bSpawnedActor = false;
		}
	}
	
	if ( !bSpawnedActor )
	{
		UE_LOGF(LogCheatManager, Warning, "Failed to summon %ls. Reason: %ls", *ClassName, *FailureReason);
	}
}

void UCheatManager::PlayersOnly()
{
	check( GetWorld() );
	if (GetWorld()->bPlayersOnly || GetWorld()->bPlayersOnlyPending)
	{
		GetWorld()->bPlayersOnly = false;
		GetWorld()->bPlayersOnlyPending = false;
	}
	else
	{
		GetWorld()->bPlayersOnlyPending = !GetWorld()->bPlayersOnlyPending;
		// World.bPlayersOnly is set after next tick of UWorld::Tick
	}	
}

void UCheatManager::ViewSelf()
{
	GetOuterAPlayerController()->ResetCameraMode();
	if ( GetOuterAPlayerController()->GetPawn() != NULL )
	{
		GetOuterAPlayerController()->SetViewTarget(GetOuterAPlayerController()->GetPawn());
	}
	else
	{
		GetOuterAPlayerController()->SetViewTarget(GetOuterAPlayerController()); //-V678
	}
	GetOuterAPlayerController()->ClientMessage(LOCTEXT("OwnCamera", "Viewing from own camera").ToString(), TEXT("Event"));
}

void UCheatManager::ViewPlayer( const FString& S )
{
	AController* Controller = NULL;
	for( FConstControllerIterator Iterator = GetWorld()->GetControllerIterator(); Iterator; ++Iterator )
	{
		Controller = Iterator->Get();
		if ( Controller->PlayerState && (FCString::Stricmp(*Controller->PlayerState->GetPlayerName(), *S) == 0 ) )
		{
			break;
		}
	}

	if ( Controller && Controller->GetPawn() != NULL )
	{
		GetOuterAPlayerController()->ClientMessage(FText::Format(LOCTEXT("ViewPlayer", "Viewing from {0}"), FText::FromString(Controller->PlayerState->GetPlayerName())).ToString(), TEXT("Event"));
		GetOuterAPlayerController()->SetViewTarget(Controller->GetPawn());
	}
}

void UCheatManager::ViewActor( FName ActorName)
{
	for (FActorIterator It(GetWorld()); It; ++It)
	{
		AActor* A = *It;
		if (IsValid(A))
		{
			if ( A->GetFName() == ActorName )
			{
				GetOuterAPlayerController()->SetViewTarget(A);
				static FName NAME_ThirdPerson = FName("ThirdPerson");
				GetOuterAPlayerController()->SetCameraMode(NAME_ThirdPerson);
				return;
			}
		}
	}
}

void UCheatManager::ViewClass( TSubclassOf<AActor> DesiredClass )
{
	bool bFound = false;
	AActor* First = NULL;
	for (TActorIterator<AActor> It(GetWorld(), DesiredClass); It; ++It)
	{
		AActor* TestActor = *It;
		if (IsValidChecked(TestActor))
		{
			AActor* Other = TestActor;
			if (bFound || (First == NULL))
			{
				First = Other;
				if (bFound)
				{
					break;
				}
			}

			if (Other == GetOuterAPlayerController()->PlayerCameraManager->GetViewTarget())
			{
				bFound = true;
			}
		}
	}

	if (First != NULL)
	{
		GetOuterAPlayerController()->ClientMessage(FText::Format(LOCTEXT("ViewPlayer", "Viewing from {0}"), FText::FromString(First->GetHumanReadableName())).ToString(), TEXT("Event"));
		GetOuterAPlayerController()->SetViewTarget(First);
	}
	else
	{
		ViewSelf();
	}
}

void UCheatManager::SetLevelStreamingStatus(FName PackageName, bool bShouldBeLoaded, bool bShouldBeVisible)
{
	UWorld* World = GetWorld();
	if (PackageName != NAME_All)
	{
		for( FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator )
		{
			if (APlayerController* PC = Iterator->Get())
			{
				PC->ClientUpdateLevelStreamingStatus(PC->NetworkRemapPath(PackageName, false), bShouldBeLoaded, bShouldBeVisible, false, INDEX_NONE);
			}
		}
	}
	else
	{
		for( FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator )
		{
			if (APlayerController* PC = Iterator->Get())
			{
				TArray<FUpdateLevelStreamingLevelStatus> LevelStatuses;
				for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
				{
					FUpdateLevelStreamingLevelStatus& LevelStatus = *new(LevelStatuses) FUpdateLevelStreamingLevelStatus();
					LevelStatus.PackageName = PC->NetworkRemapPath(StreamingLevel->GetWorldAssetPackageFName(), false);
					LevelStatus.bNewShouldBeLoaded = bShouldBeLoaded;
					LevelStatus.bNewShouldBeVisible = bShouldBeVisible;
					LevelStatus.bNewShouldBlockOnLoad = false;
					LevelStatus.bNewShouldBlockOnUnload = false;
					LevelStatus.LODIndex = INDEX_NONE;
				}
				PC->ClientUpdateMultipleLevelsStreamingStatus(LevelStatuses);
			}
		}
	}
}

void UCheatManager::StreamLevelIn(FName PackageName)
{
	SetLevelStreamingStatus(PackageName, true, true);
}

void UCheatManager::OnlyLoadLevel(FName PackageName)
{
	SetLevelStreamingStatus(PackageName, true, false);
}

void UCheatManager::StreamLevelOut(FName PackageName)
{
	SetLevelStreamingStatus(PackageName, false, false);
}

void UCheatManager::ToggleDebugCamera()
{
	if (IsDebugCameraActive())
	{
		DisableDebugCamera();
	}
	else
	{
		EnableDebugCamera();
	}
}

bool UCheatManager::IsDebugCameraActive() const
{
	return FDebugCameraManager::Get().IsDebugCameraActive(GetOuterAPlayerController());
}

void UCheatManager::EnableDebugCamera()
{
	FDebugCameraManager::Get().EnableDebugCamera(GetOuterAPlayerController());
}

void UCheatManager::DisableDebugCamera()
{
	FDebugCameraManager::Get().DisableDebugCamera(GetOuterAPlayerController());
}

void UCheatManager::InitCheatManager() 
{
	ReceiveInitCheatManager(); //BP Initialization event
	OnCheatManagerCreatedDelegate.Broadcast(this);
	GetOuterAPlayerController()->OnEndPlay.AddDynamic(this, &UCheatManager::OnPlayerEndPlayed);
}

void UCheatManager::BeginDestroy()
{
#if ENABLE_VISUAL_LOG
	if (bToggleAILogging && FVisualLogger::Get().IsRecording())
	{
		// stop recording and dump all remaining logs
		FVisualLogger::Get().SetIsRecording(false);
		FVisualLogger::Get().SetIsRecordingToFile(false);
		bToggleAILogging = false;
		FVisualLogger::Get().SetIsRecordingOnServer(false);
	}
#endif
	Super::BeginDestroy();
}

/// @cond DOXYGEN_WARNINGS

bool UCheatManager::ServerToggleAILogging_Validate()
{
	return true;
}

void UCheatManager::ServerToggleAILogging_Implementation()
{
#if ENABLE_VISUAL_LOG
	if (FVisualLogger::Get().IsRecordingToFile())
	{
		// stop recording and dump all remaining logs in a moment
		FVisualLogger::Get().SetIsRecordingToFile(false);
		FVisualLogger::Get().SetIsRecording(false);
		bToggleAILogging = false;
	}
	else
	{
		FVisualLogger::Get().SetIsRecordingToFile(true);
		bToggleAILogging = true;
	}

	FVisualLogger::Get().SetIsRecordingOnServer(bToggleAILogging);
	UWorld *World = GetWorld();
	if (World)
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PC = Iterator->Get();
			if (PC)
			{
				PC->OnServerStartedVisualLogger(bToggleAILogging);
			}
		}
	}
	else
	{
		GetOuterAPlayerController()->OnServerStartedVisualLogger(bToggleAILogging);
		GetOuterAPlayerController()->ClientMessage(FString::Printf(TEXT("VisLog recording is now %s"), FVisualLogger::Get().IsRecording() ? TEXT("Enabled") : TEXT("Disabled")));
	}
#endif
}

/// @endcond

void UCheatManager::ToggleAILogging()
{
#if ENABLE_VISUAL_LOG
	APlayerController* PC = GetOuterAPlayerController();
	if (!PC)
	{
		return;
	}

	UWorld *World = GetWorld();
	if (World && World->GetNetMode() == NM_Client)
	{
		PC->ServerToggleAILogging();
	}
	else
	{
		ServerToggleAILogging();
	}
#endif
}

#define SAFE_TRACEINDEX_DECREASE(x) ((--x)<0)? 9:(x)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void UCheatManager::TickCollisionDebug()
{
	// If we are debugging capsule tracing
	if(bDebugCapsuleSweep)
	{
		APlayerController* PC = GetOuterAPlayerController();
		if(PC)
		{
			// Get view location to act as start point
			FVector ViewLoc;
			FRotator ViewRot;
			PC->GetPlayerViewPoint(ViewLoc, ViewRot);
			FVector ViewDir = ViewRot.Vector();
			FVector End = ViewLoc + (DebugTraceDistance * ViewDir);

			// Fill in params and do trace
			FCollisionQueryParams CapsuleParams(SCENE_QUERY_STAT(TickCollisionDebug), false, PC->GetPawn());
			CapsuleParams.bTraceComplex = bDebugCapsuleTraceComplex;

			if (bDebugCapsuleSweep)
			{
				// If we get a hit, draw the capsule
				FHitResult Result;
				bool bHit = GetWorld()->SweepSingleByChannel(Result, ViewLoc, End, FQuat::Identity, DebugTraceChannel, FCollisionShape::MakeCapsule(DebugCapsuleRadius, DebugCapsuleHalfHeight), CapsuleParams);
				if(bHit)
				{
					AddCapsuleSweepDebugInfo(ViewLoc, End, Result.ImpactPoint, Result.Normal, Result.ImpactNormal, Result.Location, DebugCapsuleHalfHeight, DebugCapsuleRadius, false, (Result.bStartPenetrating && Result.bBlockingHit)? true: false);
					UE_LOGF(LogCollision, Log, "Collision component (%ls) : Actor (%ls)", *GetNameSafe(Result.Component.Get()), *Result.HitObjectHandle.GetName());
				}
			}
		}
	}

	// draw
	for (int32 TraceIdx=0; TraceIdx < DebugTraceInfoList.Num(); ++TraceIdx)
	{
		FDebugTraceInfo & TraceInfo = DebugTraceInfoList[TraceIdx];
		DrawDebugDirectionalArrow(GetWorld(), TraceInfo.LineTraceStart, TraceInfo.LineTraceEnd, 10.f, FColor::White, SDPG_World);
		// if it's current trace index, use highlight color
		if (CurrentTraceIndex == TraceIdx)
		{
			if (TraceInfo.bInsideOfObject)
			{
				DrawDebugCapsule(GetWorld(), TraceInfo.HitLocation, TraceInfo.CapsuleHalfHeight, TraceInfo.CapsuleRadius, FQuat::Identity, FColor(255,100,64));
			}
			else
			{
				DrawDebugCapsule(GetWorld(), TraceInfo.HitLocation, TraceInfo.CapsuleHalfHeight, TraceInfo.CapsuleRadius, FQuat::Identity, FColor(255,200,128));
			}
		}
		else
		{
			if (TraceInfo.bInsideOfObject)
			{
				DrawDebugCapsule(GetWorld(), TraceInfo.HitLocation, TraceInfo.CapsuleHalfHeight, TraceInfo.CapsuleRadius, FQuat::Identity, FColor(64,100,255));
			}
			else
			{
				DrawDebugCapsule(GetWorld(), TraceInfo.HitLocation, TraceInfo.CapsuleHalfHeight, TraceInfo.CapsuleRadius, FQuat::Identity, FColor(128,200,255));
			}
		}

		DrawDebugDirectionalArrow(GetWorld(), TraceInfo.HitNormalStart, TraceInfo.HitNormalEnd, 5, FColor(255,64,64), SDPG_World);

		DrawDebugDirectionalArrow(GetWorld(), TraceInfo.HitNormalStart, TraceInfo.HitImpactNormalEnd, 5, FColor(64,64,255), SDPG_World);
	}

	FLinearColor CurrentColor(255.f/255.f,96.f/255.f,96/255.f);
	FLinearColor DeltaColor = (FLinearColor(1.0f, 1.0f, 1.0f, 1.0f) - CurrentColor)*0.1f;
	int32 TotalCount=0;

	if ( DebugTracePawnInfoList.Num() > 0 )
	{
		// the latest will draw very red-ish to whiter color as it gets older. 
		for (int32 TraceIdx=CurrentTracePawnIndex; TotalCount<10; TraceIdx=SAFE_TRACEINDEX_DECREASE(TraceIdx), CurrentColor+=DeltaColor, ++TotalCount)
		{
			FDebugTraceInfo & TraceInfo = DebugTracePawnInfoList[TraceIdx];
			DrawDebugDirectionalArrow(GetWorld(), TraceInfo.LineTraceStart, TraceInfo.LineTraceEnd, 10.f, FColor(200,200,100), SDPG_World);

			if (TraceInfo.bInsideOfObject)
			{
				DrawDebugCapsule(GetWorld(), TraceInfo.HitLocation, TraceInfo.CapsuleHalfHeight, TraceInfo.CapsuleRadius, FQuat::Identity, FColor(64, 64, 255));
			}
			else
			{
				DrawDebugCapsule(GetWorld(), TraceInfo.HitLocation, TraceInfo.CapsuleHalfHeight, TraceInfo.CapsuleRadius, FQuat::Identity, CurrentColor.QuantizeRound());
			}
			DrawDebugDirectionalArrow(GetWorld(), TraceInfo.HitNormalStart, TraceInfo.HitNormalEnd, 5.f, FColor(255,64,64), SDPG_World);
		}
	}
}

void UCheatManager::AddCapsuleSweepDebugInfo(
	const FVector& LineTraceStart, 
	const FVector& LineTraceEnd, 
	const FVector& HitImpactLocation, 
	const FVector& HitNormal, 
	const FVector& HitImpactNormal, 
	const FVector& HitLocation, 
	float CapsuleHalfheight, 
	float CapsuleRadius, 
	bool bTracePawn, 
	bool bInsideOfObject  )
{
	if (bTracePawn)
	{
		// to keep the last index to be the one added. We increase index first
		// this gets initialized to be -1, so it should be 0 when it starts. Max is 10
		if (++CurrentTracePawnIndex>9)
		{
			CurrentTracePawnIndex =0;
		}
	}

	FDebugTraceInfo & TraceInfo = (bTracePawn)?DebugTracePawnInfoList[CurrentTracePawnIndex]:DebugTraceInfoList[CurrentTraceIndex];

	TraceInfo.LineTraceStart = LineTraceStart;
	TraceInfo.LineTraceEnd = LineTraceEnd;
	TraceInfo.CapsuleHalfHeight = CapsuleHalfheight;
	TraceInfo.CapsuleRadius = CapsuleRadius;
	TraceInfo.HitLocation = HitLocation;

	TraceInfo.HitNormalStart = HitImpactLocation;
	TraceInfo.HitNormalEnd = HitImpactLocation + (HitNormal*DebugTraceDrawNormalLength);
	TraceInfo.HitImpactNormalEnd = HitImpactLocation + (HitImpactNormal*DebugTraceDrawNormalLength);

	TraceInfo.bInsideOfObject = bInsideOfObject;
}
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

void UCheatManager::DebugCapsuleSweep()
{
	bDebugCapsuleSweep = !bDebugCapsuleSweep;
	if (bDebugCapsuleSweep)
	{
		CurrentTraceIndex = DebugTraceInfoList.Num();
		DebugTraceInfoList.AddUninitialized(1);
	}
	else
	{
		DebugTraceInfoList.RemoveAt(CurrentTraceIndex);
	}
}

void UCheatManager::DebugCapsuleSweepSize(float HalfHeight,float Radius)
{
	DebugCapsuleHalfHeight	= HalfHeight;
	DebugCapsuleRadius		= Radius;
}

void UCheatManager::DebugCapsuleSweepChannel(enum ECollisionChannel Channel)
{
	DebugTraceChannel = Channel;
}

void UCheatManager::DebugCapsuleSweepComplex(bool bTraceComplex)
{
	bDebugCapsuleTraceComplex = bTraceComplex;
}

void UCheatManager::DebugCapsuleSweepCapture()
{
	++CurrentTraceIndex;
	DebugTraceInfoList.AddUninitialized(1);
}

void UCheatManager::DebugCapsuleSweepPawn()
{
	bDebugCapsuleSweepPawn = !bDebugCapsuleSweepPawn;
	if (bDebugCapsuleSweepPawn)
	{
		CurrentTracePawnIndex = 0;
		// only last 10 is the one saving for Pawn
		if (DebugTracePawnInfoList.Num() == 0)
		{
			DebugTracePawnInfoList.AddUninitialized(10);
		}
	}
}

void UCheatManager::DebugCapsuleSweepClear()
{
	CurrentTraceIndex = 0;
	DebugTraceInfoList.Empty();
	DebugTracePawnInfoList.Empty();
	if (bDebugCapsuleSweep)
	{
		DebugTraceInfoList.AddUninitialized(1);
	}

	if (bDebugCapsuleSweepPawn)
	{
		CurrentTracePawnIndex = 0;
		DebugTracePawnInfoList.AddUninitialized(10);
	}
}

void UCheatManager::TestCollisionDistance()
{
#if ENABLE_DRAW_DEBUG
	APlayerController* PC = GetOuterAPlayerController();
	if(PC)
	{
		// Get view location to act as start point
		FVector ViewLoc;
		FRotator ViewRot;
		PC->GetPlayerViewPoint(ViewLoc, ViewRot);

		FlushPersistentDebugLines( GetOuterAPlayerController()->GetWorld() );//change the GetWorld

		// calculate from viewloc
		for (FThreadSafeObjectIterator Iter(AVolume::StaticClass()); Iter; ++Iter)
		{
			AVolume * Volume = Cast<AVolume>(*Iter);

			if (Volume->GetClass()->GetDefaultObject() != Volume)
			{
				FVector ClosestPoint(0,0,0);
				float Distance = Volume->GetBrushComponent()->GetDistanceToCollision(ViewLoc, ClosestPoint);
				float NormalizedDistance = FMath::Clamp<float>(Distance, 0.f, 1000.f)/1000.f;
				FColor DrawColor(255*NormalizedDistance, 255*(1-NormalizedDistance), 0);
				DrawDebugLine(GetWorld(), ViewLoc, ClosestPoint, DrawColor, true);

				UE_LOGF(LogCheatManager, Log, "Distance to (%ls) is %0.2f", *Volume->GetName(), Distance);
			}
		}
	}
#endif
}

void UCheatManager::DumpOnlineSessionState()
{
	UOnlineEngineInterface::Get()->DumpSessionState(GetWorld());
}

void UCheatManager::DumpPartyState()
{
	UOnlineEngineInterface::Get()->DumpPartyState(GetWorld());
}

void UCheatManager::DumpChatState()
{
	UOnlineEngineInterface::Get()->DumpChatState(GetWorld());
}

void UCheatManager::DumpVoiceMutingState()
{
	UE_LOGF(LogCheatManager, Display, "");
	UE_LOGF(LogCheatManager, Display, "-------------------------------------------------------------");
	UE_LOGF(LogCheatManager, Display, "");

	// Log the online view of the voice state
	UOnlineEngineInterface::Get()->DumpVoiceState(GetWorld());

	// For each player list their gameplay mutes and system wide mutes
	UE_LOGF(LogCheatManager, Display, "\n%ls", *DumpMutelistState(GetWorld()));
}

UWorld* UCheatManager::GetWorld() const
{
	if (IsTemplate())
	{
		return nullptr;
	}

	return GetOuterAPlayerController()->GetWorld();
}

void UCheatManager::BugItGo( float X, float Y, float Z, float Pitch, float Yaw, float Roll )
{
	FVector TheLocation(X, Y, Z);
	FRotator TheRotation(Pitch, Yaw, Roll);
	BugItWorker( TheLocation, TheRotation );
}


void UCheatManager::BugItGoString(const FString& TheLocation, const FString& TheRotation)
{
	const TCHAR* Stream = *TheLocation;
	FVector Vect(ForceInit);
	Vect.X = FCString::Atof(Stream);
	Stream = FCString::Strstr(Stream,TEXT(","));
	if( Stream )
	{
		Vect.Y = FCString::Atof(++Stream);
		Stream = FCString::Strstr(Stream,TEXT(","));
		if( Stream )
			Vect.Z = FCString::Atof(++Stream);
	}

	const TCHAR* RotStream = *TheRotation;
	FRotator Rotation(ForceInit);
	Rotation.Pitch = FCString::Atof(RotStream);
	RotStream = FCString::Strstr(RotStream,TEXT(","));
	if( RotStream )
	{
		Rotation.Yaw = FCString::Atof(++RotStream);
		RotStream = FCString::Strstr(RotStream,TEXT(","));
		if( RotStream )
			Rotation.Roll = FCString::Atof(++RotStream);
	}

	BugItWorker(Vect, Rotation);
}


void UCheatManager::BugItWorker( FVector TheLocation, FRotator TheRotation )
{
	UE_LOGF(LogCheatManager, Log,  "BugItGo to: %ls %ls", *TheLocation.ToString(), *TheRotation.ToString() );

	// ghost so we can go anywhere
	Ghost();

	APlayerController* const MyPlayerController = GetOuterAPlayerController();
	APawn* const MyPawn = MyPlayerController->GetPawn();
	if (MyPawn)
	{
		MyPawn->TeleportTo(TheLocation, TheRotation);
		MyPawn->FaceRotation(TheRotation, 0.0f);
	}
	MyPlayerController->SetControlRotation(TheRotation);

	// ghost again in case teleporting changed the movement mode
	Ghost();
	GetOuterAPlayerController()->ClientMessage(TEXT("BugItGo: Ghost mode is ON"));
}


void UCheatManager::BugIt( const FString& ScreenShotDescription )
{
	APlayerController* const MyPlayerController = GetOuterAPlayerController();

	// Path will be <gamename>/bugit/<platform>/desc/desc_ (BugItDir() includes a platform and trailing slash)
	const FString BaseFile = FString::Printf(TEXT("%s%s/%s_"), *FPaths::BugItDir(), *ScreenShotDescription, *ScreenShotDescription);
	FString ScreenShotFile;

	// find the next filename in the sequence, e.g <gamename>/bugit/<platform>/desc_00000.png
	FFileHelper::GenerateNextBitmapFilename(BaseFile, TEXT("png"), ScreenShotFile);

	// request a screenshot to that path
	MyPlayerController->ConsoleCommand( FString::Printf(TEXT("BUGSCREENSHOTWITHHUDINFO %s"), *ScreenShotFile));

	FVector ViewLocation;
	FRotator ViewRotation;
	MyPlayerController->GetPlayerViewPoint( ViewLocation, ViewRotation );

	if( MyPlayerController->GetPawn() != NULL )
	{
		ViewLocation = MyPlayerController->GetPawn()->GetActorLocation();
	}

	FString GoString, LocString;
	BugItStringCreator( ViewLocation, ViewRotation, GoString, LocString );

	// Log bugit data to a textfile with the same name as the screenshot
	LogOutBugItGoToLogFile( ScreenShotDescription, ScreenShotFile, GoString, LocString );
}

void UCheatManager::BugItStringCreator( FVector ViewLocation, FRotator ViewRotation, FString& GoString, FString& LocString )
{
	GoString = FString::Printf(TEXT("BugItGo %f %f %f %f %f %f"), ViewLocation.X, ViewLocation.Y, ViewLocation.Z, ViewRotation.Pitch, ViewRotation.Yaw, ViewRotation.Roll);
	UE_LOGF(LogCheatManager, Log, "%ls", *GoString);

	LocString = FString::Printf(TEXT("?BugLoc=%s?BugRot=%s"), *ViewLocation.ToString(), *ViewRotation.ToString());
	UE_LOGF(LogCheatManager, Log, "%ls", *LocString);
}

void UCheatManager::FlushLog()
{
	GLog->FlushThreadedLogs();
	GLog->Flush();
}

void UCheatManager::LogLoc()
{
	APlayerController* const MyPlayerController = GetOuterAPlayerController();

	FVector ViewLocation;
	FRotator ViewRotation;
	MyPlayerController->GetPlayerViewPoint( ViewLocation, ViewRotation );
	if( MyPlayerController->GetPawn() != NULL )
	{
		ViewLocation = MyPlayerController->GetPawn()->GetActorLocation();
	}
	FString GoString, LocString;
	BugItStringCreator( ViewLocation, ViewRotation, GoString, LocString );
}

void UCheatManager::SetWorldOrigin()
{
	UWorld* World = GetWorld();
	check(World);

	APlayerController* const MyPlayerController = GetOuterAPlayerController();

	FVector ViewLocation;
	FRotator ViewRotation;
	MyPlayerController->GetPlayerViewPoint( ViewLocation, ViewRotation );
	if( MyPlayerController->GetPawn() != NULL )
	{
		ViewLocation = MyPlayerController->GetPawn()->GetActorLocation();
	}
	
	// Consider only XY plane
	ViewLocation.Z = 0;

	FIntVector NewOrigin = FIntVector(ViewLocation.X, ViewLocation.Y, ViewLocation.Z) + World->OriginLocation;
	World->RequestNewWorldOrigin(NewOrigin);
}

void UCheatManager::SetMouseSensitivityToDefault()
{
	UPlayerInput* PlayerInput = GetOuterAPlayerController()->PlayerInput;
	if (PlayerInput)
	{
		// find default sensitivity restore to that
		for (const FInputAxisConfigEntry& AxisConfigEntry : GetDefault<UInputSettings>()->AxisConfig)
		{
			const FKey AxisKey = AxisConfigEntry.AxisKeyName;
			if (AxisKey == EKeys::MouseX)
			{
				PlayerInput->SetMouseSensitivity(AxisConfigEntry.AxisProperties.Sensitivity,PlayerInput->GetMouseSensitivityY());
			}
			else if (AxisKey == EKeys::MouseY)
			{
				PlayerInput->SetMouseSensitivity(PlayerInput->GetMouseSensitivityX(),AxisConfigEntry.AxisProperties.Sensitivity);
			}
		}
	}
}

void UCheatManager::InvertMouse()
{
	UPlayerInput* PlayerInput = GetOuterAPlayerController()->PlayerInput;
	if (PlayerInput)
	{
		PlayerInput->InvertAxisKey(EKeys::MouseY);
	}
}

void UCheatManager::CheatScript(FString ScriptName)
{
	APlayerController* const PlayerController = GetOuterAPlayerController();
	ULocalPlayer* const LocalPlayer = PlayerController ? Cast<ULocalPlayer>(PlayerController->Player) : nullptr;

	UConsole* ConsoleToDisplayResults = (LocalPlayer && LocalPlayer->ViewportClient) ? LocalPlayer->ViewportClient->ViewportConsole : nullptr;

	// Run commands from the ini
	const FConfigSection* CommandsToRun = GConfig->GetSection(*FString::Printf(TEXT("CheatScript.%s"), *ScriptName), 0, GGameIni);

	if (CommandsToRun)
	{
		for (FConfigSectionMap::TConstIterator It(*CommandsToRun); It; ++It)
		{
			// show user what commands ran
			if (ConsoleToDisplayResults)
			{
				FString const S = FString::Printf(TEXT("> %s"), *It.Value().GetValue());
				ConsoleToDisplayResults->OutputText(S);
			}

			PlayerController->ConsoleCommand(*It.Value().GetValue(), /*bWriteToLog=*/ true);
		}
	}
	else
	{
		UE_LOGF(LogCheatManager, Warning, "Can't find section 'CheatScript.%ls' in DefaultGame.ini", *ScriptName);
	}
}

void UCheatManager::LogOutBugItGoToLogFile( const FString& InScreenShotDesc, const FString& InScreenShotPath, const FString& InGoString, const FString& InLocString )
{
#if ALLOW_DEBUG_FILES
	// Create folder if not already there (screenshot is deferred 1-frame so will not be there yet)
	IFileManager::Get().MakeDirectory( *FPaths::GetPath(InScreenShotPath));

	// Create file for log data - remove the extension from the screenshot and create a .txt path
	const FString BaseFileName = FPaths::GetBaseFilename(InScreenShotPath, false);
	const FString FullFileName = BaseFileName + TEXT(".txt");

	FOutputDeviceFile OutputFile(*FullFileName);
	
	OutputFile.Logf( TEXT("Dumping BugIt data chart at %s using build %s built from changelist %i"), *FDateTime::Now().ToString(), FApp::GetBuildVersion(), GetChangeListNumberForPerfTesting() );

	const FString MapNameStr = GetWorld()->GetMapName();

	OutputFile.Logf( TEXT("MapName: %s"), *MapNameStr );

	OutputFile.Logf( TEXT("Description: %s"), *InScreenShotDesc );
	OutputFile.Logf( TEXT("%s"), *InGoString );
	OutputFile.Logf( TEXT("%s"), *InLocString );

	OutputFile.Logf( TEXT(" ---=== GameSpecificData ===--- ") );
	DoGameSpecificBugItLog(OutputFile);

	// Flush, close and delete.
	//delete OutputFile;
	OutputFile.TearDown();

#if PLATFORM_DESKTOP
	FPlatformApplicationMisc::ClipboardCopy(*InGoString);
#endif

	// so here we want to send this bad boy back to the PC
	SendDataToPCViaUnrealConsole( TEXT("UE_PROFILER!BUGIT:"), *(FullFileName) );
#endif // ALLOW_DEBUG_FILES
}

bool UCheatManager::DoGameSpecificBugItLog(FOutputDevice& OutputFile)
{
	for (UCheatManagerExtension* Extension : CheatManagerExtensions)
	{
		if (Extension)
		{
			Extension->DoExtensionSpecificBugItLog(OutputFile);
		}
	}

	return true;
}

AActor* UCheatManager::GetTarget(APlayerController* PlayerController, struct FHitResult& OutHit)
{
    if ((PlayerController == NULL) || (PlayerController->PlayerCameraManager == NULL))
    {
        return NULL;
    }
    
    check(GetWorld() != NULL);
    FVector const CamLoc = PlayerController->PlayerCameraManager->GetCameraLocation();
    FRotator const CamRot = PlayerController->PlayerCameraManager->GetCameraRotation();
    
    FCollisionQueryParams TraceParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), true, PlayerController->GetPawn());
    bool bHit = GetWorld()->LineTraceSingleByChannel(OutHit, CamLoc, CamRot.Vector() * 100000.f + CamLoc, ECC_Pawn, TraceParams);
    if (bHit)
    {
        check(OutHit.HitObjectHandle.FetchActor() != nullptr);
		return OutHit.HitObjectHandle.FetchActor();
    }
    return NULL;
}


void UCheatManager::SpawnServerStatReplicator()
{
	DestroyServerStatReplicator();

	AGameModeBase* GameModeBase = GetWorld()->GetAuthGameMode();
	if (GameModeBase != nullptr && GameModeBase->IsNetMode(NM_DedicatedServer))
	{
		check(GameModeBase->ServerStatReplicator == nullptr);

		APlayerController* PlayerController = GetOuterAPlayerController();
		if (PlayerController->NetConnection != nullptr &&
			PlayerController->NetConnection->Driver != nullptr)
		{
			PlayerController->NetConnection->Driver->bCollectNetStats = true;
		}
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.Name = FName("ServerStatReplicatorInst");
		SpawnInfo.Owner = PlayerController;
		GameModeBase->ServerStatReplicator = GetWorld()->SpawnActor<AServerStatReplicator>(SpawnInfo);
		UE_LOGF(LogCheatManager, Log, "Spawned stat replicator (%ls) for owner (%ls)",
			*GameModeBase->ServerStatReplicator->GetName(), *PlayerController->GetName());
	}
}

void UCheatManager::DestroyServerStatReplicator()
{
	AGameModeBase* GameModeBase = GetWorld()->GetAuthGameMode();
	if (GameModeBase != nullptr && GameModeBase->ServerStatReplicator != nullptr)
	{
		UE_LOGF(LogCheatManager, Log, "Destroying stat replicator (%ls)", *GameModeBase->ServerStatReplicator->GetName());
		GameModeBase->ServerStatReplicator->Destroy(true);
		GameModeBase->ServerStatReplicator = nullptr;

		// Toggle off stats collection
		APlayerController* PlayerController = GetOuterAPlayerController();
		if (PlayerController->NetConnection != nullptr &&
			PlayerController->NetConnection->Driver != nullptr)
		{
			PlayerController->NetConnection->Driver->bCollectNetStats = false;
		}
	}
}

void UCheatManager::ToggleServerStatReplicatorClientOverwrite()
{
	APlayerController* PlayerController = GetOuterAPlayerController();
	if (ensure(PlayerController))
	{
		AServerStatReplicator* ServerStatReplicator = FindObject<AServerStatReplicator>(PlayerController->GetLevel(), TEXT("ServerStatReplicatorInst"));
		if (ServerStatReplicator != nullptr)
		{
			ServerStatReplicator->bOverwriteClientStats = !ServerStatReplicator->bOverwriteClientStats;
		}
	}
}

void UCheatManager::ToggleServerStatReplicatorUpdateStatNet()
{
	APlayerController* PlayerController = GetOuterAPlayerController();
	if (ensure(PlayerController))
	{
		AServerStatReplicator* ServerStatReplicator = FindObject<AServerStatReplicator>(PlayerController->GetLevel(), TEXT("ServerStatReplicatorInst"));
		if (ServerStatReplicator != nullptr)
		{
			ServerStatReplicator->bUpdateStatNet = !ServerStatReplicator->bUpdateStatNet;
		}
	}
}

void UCheatManager::UpdateSafeArea()
{
	FCoreDelegates::OnSafeFrameChangedEvent.Broadcast();
}

void UCheatManager::AddCheatManagerExtension(UCheatManagerExtension* CheatObject)
{
	if (ensure(CheatObject))
	{
		if (!CheatManagerExtensions.Contains(CheatObject))
		{
			CheatManagerExtensions.Add(CheatObject);
			CheatObject->AddedToCheatManager();
		}
	}
}

void UCheatManager::RemoveCheatManagerExtension(UCheatManagerExtension* CheatObject)
{
	if (ensure(CheatObject))
	{
		int32 CheatExtensionIdx = CheatManagerExtensions.IndexOfByKey(CheatObject);
		if (CheatExtensionIdx != INDEX_NONE)
		{
			CheatManagerExtensions.RemoveAt(CheatExtensionIdx);
			CheatObject->RemovedFromCheatManager();
		}
	}
}

UCheatManagerExtension* UCheatManager::FindCheatManagerExtension(const UClass* InClass) const
{
	for (UCheatManagerExtension* CheatObject : CheatManagerExtensions)
	{
		if (CheatObject && CheatObject->IsA(InClass))
		{
			return CheatObject;
		}
	}

	return nullptr;
}

UCheatManagerExtension* UCheatManager::FindCheatManagerExtensionInterface(const UClass* InClass) const
{
	for (UCheatManagerExtension* CheatObject : CheatManagerExtensions)
	{
		if (CheatObject && CheatObject->GetClass()->ImplementsInterface(InClass))
		{
			return CheatObject;
		}
	}

	return nullptr;
}

FDelegateHandle UCheatManager::RegisterForOnCheatManagerCreated(FOnCheatManagerCreated::FDelegate&& Delegate)
{
	for (TObjectIterator<UCheatManager> CheatIt; CheatIt; ++CheatIt)
	{
		UCheatManager* CheatInstance = *CheatIt;
		if (CheatInstance->GetOuter()->IsA(APlayerController::StaticClass()))
		{
			Delegate.Execute(CheatInstance);
		}
	}

	// Register for cheat managers created in the future
	return OnCheatManagerCreatedDelegate.Add(MoveTemp(Delegate));
}

void UCheatManager::UnregisterFromOnCheatManagerCreated(FDelegateHandle DelegateHandle)
{
	OnCheatManagerCreatedDelegate.Remove(DelegateHandle);
}

APlayerController* UCheatManager::GetPlayerController() const
{
	return GetOuterAPlayerController();
}

#undef LOCTEXT_NAMESPACE

