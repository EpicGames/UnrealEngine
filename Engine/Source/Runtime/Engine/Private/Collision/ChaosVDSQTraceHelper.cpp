// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSQTraceHelper.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "ChaosVDRuntimeModule.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"


#include "ChaosVisualDebugger/ChaosVDContextProvider.h"
#include "ChaosVisualDebugger/ChaosVDDataWrapperUtils.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#include "PBDRigidsSolver.h"
#include "Physics/PhysicsGeometry.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/PhysicsLogUtil.h"

namespace Chaos::VisualDebugger::TraceHelpers
{

/** This is not great but all ignored actors/components in the collision query params struct are stores as the UObject internal index,
 * so we need to do this reverse look up to get the actual object to be able to extract the name or access the particles, so then we can record it */
template<typename TObject>
const TObject* GetObjetPointerFromID(int32 ID)
{
	constexpr  bool bEvenIfGarbage = false;
	const FUObjectItem* ObjectItem = GUObjectArray.IndexToValidObject(ID, bEvenIfGarbage);
	if (const TObject* CastedObject = Cast<TObject>(ObjectItem ? static_cast<const UObject*>(ObjectItem->GetObject()) : nullptr))
	{
		return CastedObject;
	}

	return nullptr;
}

int32 GetWorldSolverID(const UWorld* World)
	{
		if (Chaos::FPhysicsSolver* Solver = World->GetPhysicsScene() ? World->GetPhysicsScene()->GetSolver() : nullptr)
		{
			return  Solver->GetChaosVDContextData().Id;
		}

		return INDEX_NONE;
	}

	FChaosVDContext CreateSceneQueryContextHelper()
	{
		if (!FChaosVisualDebuggerTrace::IsTracing())
		{
			return FChaosVDContext();
		}

		FChaosVDContext CurrentCVDContext;
		CVD_GET_CURRENT_CONTEXT(CurrentCVDContext)
		
		FChaosVDContext NewSceneQueryContext;
		NewSceneQueryContext.Id = FChaosVDRuntimeModule::Get().GenerateUniqueID();
		NewSceneQueryContext.SetDataChannel(CVDDC_SceneQueries);

		if (CurrentCVDContext.Type ==  static_cast<int32>(EChaosVDContextType::Query) || CurrentCVDContext.Type == static_cast<int32>(EChaosVDContextType::SubTraceQuery))
		{
			NewSceneQueryContext.OwnerID = CurrentCVDContext.Id;
			NewSceneQueryContext.Type = static_cast<int32>(EChaosVDContextType::SubTraceQuery);
		}
		else
		{
			NewSceneQueryContext.Type = static_cast<int32>(EChaosVDContextType::Query);
		}

		return NewSceneQueryContext;
	}

	void TraceCVDSceneQueryStartHelper(const UWorld* World, const FPhysicsGeometry* Geom, const FTransform& StartGeomPose, const FVector& EndLocation, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams, EChaosVDSceneQueryType Type, EChaosVDSceneQueryMode Mode, bool bIsRetry)
	{
		if (!FChaosVisualDebuggerTrace::IsTracing())
		{
			return;
		}

		//TODO: See if we can move most of the code here to the Trace class of CVD using templates, like we do already for other similar cases

		FChaosVDCollisionQueryParams CVDCollisionParams;
		CVDCollisionParams.CopyFrom(Params);

		CVDCollisionParams.IgnoredActorsNames.Reserve(Params.GetIgnoredSourceObjects().Num());
		CVDCollisionParams.IgnoredComponentsNames.Reserve(Params.GetIgnoredComponents().Num());
		Algo::Transform(Params.GetIgnoredSourceObjects(), CVDCollisionParams.IgnoredActorsNames, [](uint32 ID){ return GetFNameSafe(GetObjetPointerFromID<AActor>(ID)); } );
		Algo::Transform(Params.GetIgnoredComponents(), CVDCollisionParams.IgnoredComponentsNames, [](uint32 ID){ return GetFNameSafe(GetObjetPointerFromID<UActorComponent>(ID)); } );

		FChaosVDCollisionResponseParams CVDCollisionResponseParams;
		CVDCollisionResponseParams.CopyFrom(ResponseParams);
		FChaosVDCollisionObjectQueryParams CVDCollisionObjectParams;
		CVDCollisionObjectParams.CopyFrom(ObjectParams);

		CVD_TRACE_SCENE_QUERY_START(Geom, StartGeomPose.GetRotation(), StartGeomPose.GetLocation(), EndLocation, TraceChannel, MoveTemp(CVDCollisionParams), MoveTemp(CVDCollisionResponseParams), MoveTemp(CVDCollisionObjectParams), Type, Mode, TraceHelpers::GetWorldSolverID(World), bIsRetry);
	}
}

#endif
	
