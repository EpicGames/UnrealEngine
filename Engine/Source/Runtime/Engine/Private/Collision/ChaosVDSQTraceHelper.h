// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_CHAOS_VISUAL_DEBUGGER

#include "ChaosVisualDebugger/ChaosVDContextProvider.h"
#include "HAL/Platform.h"
#include "Math/Transform.h"
#include "PhysicsInterfaceDeclaresCore.h"

class UWorld;

enum ECollisionChannel: int;
enum class EChaosVDSceneQueryType;
enum class EChaosVDSceneQueryMode;

struct FCollisionQueryParams;
struct FCollisionResponseParams;
struct FCollisionObjectQueryParams;

/** Initializes a CVD Wrapper for a Scene Query entry with the provider data, and traces it immediately but it keeps scope the CVD context active until it goes out scope (so other operations like SQ Visits can be recorded under the same context) */
#ifndef CVD_TRACE_SCOPED_SCENE_QUERY_HELPER
	#define CVD_TRACE_SCOPED_SCENE_QUERY_HELPER(World, Geom, StartGeomPose, EndLocation, TraceChannel, Params,  ResponseParams, ObjectParams, Type, Mode, bIsRetry) \
	CVD_SCOPE_CONTEXT(Chaos::VisualDebugger::TraceHelpers::CreateSceneQueryContextHelper());\
	Chaos::VisualDebugger::TraceHelpers::TraceCVDSceneQueryStartHelper(World, Geom, StartGeomPose, EndLocation, TraceChannel, Params,  ResponseParams, ObjectParams, Type, Mode, bIsRetry);
#endif

namespace Chaos::VisualDebugger::TraceHelpers
{
	int32 GetWorldSolverID(const UWorld* World);

	/** Creates a CVD context with all the data required to trace a scene query */
	FChaosVDContext CreateSceneQueryContextHelper();

	/** Process types not accessible in the Chaos module, and call the CVD trace to perform the trace of the scene query */
	void TraceCVDSceneQueryStartHelper(const UWorld* World, const FPhysicsGeometry* Geom, const FTransform& StartGeomPose, const FVector& EndLocation, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams, EChaosVDSceneQueryType Type, EChaosVDSceneQueryMode Mode, bool bIsRetry);
}

#else // WITH_CHAOS_VISUAL_DEBUGGER
	#ifndef CVD_TRACE_SCOPED_SCENE_QUERY_HELPER
		#define CVD_TRACE_SCOPED_SCENE_QUERY_HELPER(World, Geom, StartGeomPose, EndLocation, TraceChannel, Params,  ResponseParams, ObjectParams, Type, Mode, bIsRetry)
	#endif
#endif // WITH_CHAOS_VISUAL_DEBUGGER