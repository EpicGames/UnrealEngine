// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Collision.cpp: AActor collision implementation
=============================================================================*/

#include "Collision.h"
#include "Engine/SkinnedAsset.h"
#include "UObject/UObjectIterator.h"
#include "Engine/World.h"
#include "Components/ShapeComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/CollisionProfile.h"
#include "BodySetupEnums.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsSettings.h"

// TEMP until crash is fixed in IsCollisionEnabled().
#include "LandscapeHeightfieldCollisionComponent.h"

//////////////////////////////////////////////////////////////////////////
// FCollisionQueryFlag

FCollisionQueryFlag& FCollisionQueryFlag::Get()
{
	static FCollisionQueryFlag CollisionQueryFlag;
	return CollisionQueryFlag;
}

//////////////////////////////////////////////////////////////////////////
// FCollisionQueryParams

FCollisionQueryParams::FCollisionQueryParams(FName InTraceTag, const TStatId& InStatId, bool bInTraceComplex, const AActor* InIgnoreActor)
{
	bTraceComplex = bInTraceComplex;
	MobilityType = EQueryMobilityType::Any;
	TraceTag = InTraceTag;
	StatId = InStatId;
	bFindInitialOverlaps = true;
	bReturnFaceIndex = false;
	bReturnPhysicalMaterial = false;
	bComponentListUnique = true;
	IgnoreMask = 0;
	bIgnoreBlocks = false;
	bIgnoreTouches = false;
	bSkipNarrowPhase = false;
	AddIgnoredActor(InIgnoreActor);
	if (InIgnoreActor != NULL)
	{
		OwnerTag = InIgnoreActor->GetFName();
	}

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
		bDebugQuery = false;
#endif
	bTraceIntoSubComponents = true;
	bReplaceHitWithSubComponents = true;
}


static FORCEINLINE_DEBUGGABLE bool IsQueryCollisionEnabled(const UPrimitiveComponent* PrimComponent)
{
	const ECollisionEnabled::Type CollisionEnabled = PrimComponent->GetCollisionEnabled();

	return CollisionEnabledHasQuery(CollisionEnabled);
}

static FORCEINLINE_DEBUGGABLE bool CheckForCollision(const AActor* Actor)
{
	// Note: This code operates differently in the editor because the actors in the editor can have their collision setting become out of sync with physx.  This happens because even when collision is disabled on an actor, in the editor we 
	// tell phsx that we still require queries( See FBodyInstance::UpdatePhysicsFilterData).  Doing so allows us to perform editor only traces against objects with collision disabled.  Due to this, we cannot assume that the collision enabled flag here
	// setting is correct compared to physx and so we still ignore specified components regardless of their collision setting
#if WITH_EDITOR
	bool bCheckForCollision = false;
	if (Actor)
	{
		const UWorld* World = Actor->GetWorld();
		bCheckForCollision = World && World->IsGameWorld();
	}
#else
	const bool bCheckForCollision = true;
#endif
	return bCheckForCollision;
}


static FORCEINLINE_DEBUGGABLE bool CheckForCollision(const UPrimitiveComponent* PrimComponent)
{
	// Note: This code operates differently in the editor because the actors in the editor can have their collision setting become out of sync with physx.  This happens because even when collision is disabled on an actor, in the editor we 
	// tell phsx that we still require queries( See FBodyInstance::UpdatePhysicsFilterData).  Doing so allows us to perform editor only traces against objects with collision disabled.  Due to this, we cannot assume that the collision enabled flag here
	// setting is correct compared to physx and so we still ignore specified components regardless of their collision setting
#if WITH_EDITOR
	bool bCheckForCollision = false;
	if (PrimComponent)
	{
		const UWorld* World = PrimComponent->GetWorld();
		bCheckForCollision = World && World->IsGameWorld();
	}
#else
	const bool bCheckForCollision = true;
#endif
	return bCheckForCollision;
}

void FCollisionQueryParams::Internal_AddIgnoredSourceObject(const FWeakObjectPtr& WeakPtr)
{
	if (WeakPtr.IsValid())
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		// We only want to add the object to the ignore list if it's owned by the current server. 
		// Even if it's owned, it might currently be remote which would cause a migration upon dereference. 
		// We can grab the object directly, even if it's stale, to get the unique id and avoid a migration. 
		// This unique id will always be valid to use.
		if (UE::RemoteObject::Handle::IsOwned(WeakPtr.GetRemoteId()))
		{
			constexpr bool bEvenIfGarbage = true;
			constexpr bool bOnGameThread = true;
			check(WeakPtr.IsValid(bEvenIfGarbage, bOnGameThread)); // this should always be true if the object is owned

			// This is the UObject pointer, even if it's stale (e.g. contains some invalid data)
			const UObject* DoNotMutateLocalCopy = StaticFindObjectFastInternal(WeakPtr.GetRemoteId());

			// This is the array index in GUObjectArray which is the same as its UniqueId.
			const int32 UniqueId = GUObjectArray.ObjectToIndex(DoNotMutateLocalCopy);
			AddIgnoredSourceObject(UniqueId);
		}
		IgnoreSourceObjectReferences.Add(FRemoteObjectReference(WeakPtr));
#else
		UObject * Obj = WeakPtr.Get();
		// On rare occasions, this is null even with the above `IsValid` check
		if (Obj != nullptr)
		{
			AddIgnoredSourceObject(Obj->GetUniqueID());
		}
#endif
	}
}

void FCollisionQueryParams::AddIgnoredActor(const AActor* InIgnoreActor)
{
	AddIgnoredSourceObject(InIgnoreActor);
}

void FCollisionQueryParams::AddIgnoredActor(const uint32 InIgnoreActorID)
{
	AddIgnoredSourceObject(InIgnoreActorID);
}

void FCollisionQueryParams::AddIgnoredActors(const TArray<AActor*>& InIgnoreActors)
{
	for (int32 Idx = 0; Idx < InIgnoreActors.Num(); ++Idx)
	{
		AddIgnoredActor(InIgnoreActors[Idx]);
	}
}

void FCollisionQueryParams::AddIgnoredActors(const TArray<const AActor*>& InIgnoreActors)
{
	for (int32 Idx = 0; Idx < InIgnoreActors.Num(); ++Idx)
	{
		AddIgnoredActor(InIgnoreActors[Idx]);
	}
}

void FCollisionQueryParams::AddIgnoredActors(const TArray<TWeakObjectPtr<const AActor> >& InIgnoreActors)
{
	for (int32 Idx = 0; Idx < InIgnoreActors.Num(); ++Idx)
	{
		Internal_AddIgnoredSourceObject(static_cast<FWeakObjectPtr>(InIgnoreActors[Idx]));
	}
}

void FCollisionQueryParams::AddIgnoredSourceObject(const uint32 InIgnoreActorID)
{
	IgnoreSourceObjects.Add(InIgnoreActorID);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	UObject* Obj = (UObject*)GUObjectArray.IndexToObject(InIgnoreActorID)->GetObject();
	IgnoreSourceObjectReferences.Add(FRemoteObjectReference(Obj));
#endif
}

void FCollisionQueryParams::AddIgnoredSourceObject(const UObject* InIgnoreObject)
{
	if (InIgnoreObject)
	{
		AddIgnoredSourceObject(InIgnoreObject->GetUniqueID());
#if UE_WITH_REMOTE_OBJECT_HANDLE
		IgnoreSourceObjectReferences.Add(FRemoteObjectReference(InIgnoreObject));
#endif
	}
}

void FCollisionQueryParams::AddIgnoredSourceObject(const TWeakObjectPtr<const UObject>& InIgnoreObject)
{
	Internal_AddIgnoredSourceObject(static_cast<FWeakObjectPtr>(InIgnoreObject));
}

void FCollisionQueryParams::AddIgnoredSourceObject(const FRemoteObjectReference& ObjectRef)
{
	Internal_AddIgnoredSourceObject(ObjectRef.ToWeakPtr());
}

void FCollisionQueryParams::AddIgnoredSourceObjects(const TArray<UObject*>& InIgnoreObjects)
{
	for (int32 Idx = 0; Idx < InIgnoreObjects.Num(); ++Idx)
	{
		AddIgnoredSourceObject(InIgnoreObjects[Idx]);
	}
}

void FCollisionQueryParams::AddIgnoredSourceObjects(const TArray<const UObject*>& InIgnoreObjects)
{
	for (int32 Idx = 0; Idx < InIgnoreObjects.Num(); ++Idx)
	{
		AddIgnoredSourceObject(InIgnoreObjects[Idx]);
	}
}

void FCollisionQueryParams::AddIgnoredSourceObjects(const TArray<TWeakObjectPtr<const UObject>>& InIgnoreObjects)
{
	for (int32 Idx = 0; Idx < InIgnoreObjects.Num(); ++Idx)
	{
		AddIgnoredSourceObject(InIgnoreObjects[Idx]);
	}
}

FORCEINLINE_DEBUGGABLE void FCollisionQueryParams::Internal_AddIgnoredComponent(const UPrimitiveComponent* InIgnoreComponent)
{
	if (InIgnoreComponent && (!CheckForCollision(InIgnoreComponent) || IsQueryCollisionEnabled(InIgnoreComponent)))
	{
		Internal_AddIgnoredComponent(InIgnoreComponent->GetUniqueID());
	}
}

void FCollisionQueryParams::AddIgnoredComponent(const UPrimitiveComponent* InIgnoreComponent)
{
	Internal_AddIgnoredComponent(InIgnoreComponent);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (InIgnoreComponent != nullptr)
	{
		IgnoreComponentReferences.Add(FRemoteObjectReference(InIgnoreComponent));
	}
#endif
}

void FCollisionQueryParams::AddIgnoredComponent(const TWeakObjectPtr<UPrimitiveComponent>& InIgnoreComponent)
{
	if (InIgnoreComponent.IsValid())
	{
		bool bIsValidIgnore = true;

#if UE_WITH_REMOTE_OBJECT_HANDLE
		FRemoteObjectReference RemoteObjectRef(InIgnoreComponent);
		IgnoreComponentReferences.Add(RemoteObjectRef);
		// Only ignore objects that belong to this server's physics world (only true if owned).
		bIsValidIgnore = UE::RemoteObject::Handle::IsOwned(RemoteObjectRef.GetRemoteId());
#endif

		if (bIsValidIgnore)
		{
			Internal_AddIgnoredComponent(InIgnoreComponent.Get());
		}
	}
}

void FCollisionQueryParams::AddIgnoredComponent(const FRemoteObjectReference& InComponentRef)
{
	FWeakObjectPtr WeakPtr = InComponentRef.ToWeakPtr();
	if (WeakPtr.IsValid())
	{
		bool bIsValidIgnore = true;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		IgnoreComponentReferences.Add(InComponentRef);
		// Only ignore objects that belong to this server's physics world (only true if owned).
		bIsValidIgnore = UE::RemoteObject::Handle::IsOwned(InComponentRef.GetRemoteId());
#endif
		if (bIsValidIgnore)
		{
			UObject* Obj = WeakPtr.Get();
			// On rare occasions, this is null even with the above `IsValid` check
			if (Obj != nullptr)
			{
				UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Obj);
				check(PrimitiveComponent);
				Internal_AddIgnoredComponent(PrimitiveComponent);
			}
		}
	}
}

void FCollisionQueryParams::AddIgnoredComponents(const TArray<UPrimitiveComponent*>& InIgnoreComponents)
{
	for (const UPrimitiveComponent* IgnoreComponent : InIgnoreComponents)
	{
		AddIgnoredComponent(IgnoreComponent);
	}
}

void FCollisionQueryParams::AddIgnoredComponents(const TArray<TWeakObjectPtr<UPrimitiveComponent>>& InIgnoreComponents)
{
	for (TWeakObjectPtr<UPrimitiveComponent> IgnoreComponent : InIgnoreComponents)
	{
		AddIgnoredComponent(IgnoreComponent);
	}
}

void FCollisionQueryParams::AddIgnoredComponent_LikelyDuplicatedRoot(const UPrimitiveComponent* InIgnoreComponent)
{
	if (InIgnoreComponent && (!CheckForCollision(InIgnoreComponent) || IsQueryCollisionEnabled(InIgnoreComponent)))
	{
		// Code calling this is usually just making sure they don't add the root component to queries right before the actual query.
		// We try to avoid invalidating the uniqueness of the array if this is the case.
		const uint32 ComponentID = InIgnoreComponent->GetUniqueID();
		if (IgnoreComponents.Num() == 0 || IgnoreComponents[0] != ComponentID)
		{
			IgnoreComponents.Add(ComponentID);
			bComponentListUnique = false;
#if UE_WITH_REMOTE_OBJECT_HANDLE
			IgnoreComponentReferences.Add(FRemoteObjectReference(InIgnoreComponent));
#endif
		}
	}
}

void FCollisionQueryParams::Internal_AddIgnoredComponent(const uint32 ComponentID)
{
	IgnoreComponents.Add(ComponentID);
	bComponentListUnique = false;
}

const FCollisionQueryParams::IgnoreComponentsArrayType& FCollisionQueryParams::GetIgnoredComponents() const
{
	if (!bComponentListUnique)
	{
		// Make unique
		bComponentListUnique = true;
		if (IgnoreComponents.Num() > 1)
		{
			// For adding a collection to ignore it's faster to sort and remove duplicates
			// than to check for duplicates at each addition.
			IgnoreComponents.Sort();
			uint32* U = IgnoreComponents.GetData();
			uint32* D = U+1;
			uint32* E = U + IgnoreComponents.Num();
			while (D != E)
			{
				if (*U != *D)
				{
					*(++U) = *D;
				}
				D += 1;
			}
			IgnoreComponents.SetNum(U - IgnoreComponents.GetData() + 1, EAllowShrinking::No);
		}
	}

	return IgnoreComponents;
}

void FCollisionQueryParams::SetNumIgnoredComponents(int32 NewNum)
{
	if (NewNum > 0)
	{
		// We can only make it smaller (and uniqueness does not change).
		if (NewNum < IgnoreComponents.Num())
		{
			IgnoreComponents.SetNum(NewNum, EAllowShrinking::No);
		}
	}
	else
	{
		ClearIgnoredComponents();
	}
}

//////////////////////////////////////////////////////////////////////////
// FSeparatingAxisPointCheck

bool FSeparatingAxisPointCheck::TestSeparatingAxisCommon(const FVector& Axis, float ProjectedPolyMin, float ProjectedPolyMax)
{
	const float ProjectedCenter = FVector::DotProduct(Axis, BoxCenter);
	const float ProjectedExtent = FVector::DotProduct(Axis.GetAbs(), BoxExtent);
	const float ProjectedBoxMin = ProjectedCenter - ProjectedExtent;
	const float ProjectedBoxMax = ProjectedCenter + ProjectedExtent;

	if (ProjectedPolyMin > ProjectedBoxMax || ProjectedPolyMax < ProjectedBoxMin)
	{
		return false;
	}

	if (bCalcLeastPenetration)
	{
		const float AxisMagnitudeSqr = Axis.SizeSquared();
		if (AxisMagnitudeSqr > (UE_SMALL_NUMBER * UE_SMALL_NUMBER))
		{
			const float InvAxisMagnitude = FMath::InvSqrt(AxisMagnitudeSqr);
			const float MinPenetrationDist = (ProjectedBoxMax - ProjectedPolyMin) * InvAxisMagnitude;
			const float	MaxPenetrationDist = (ProjectedPolyMax - ProjectedBoxMin) * InvAxisMagnitude;

			if (MinPenetrationDist < BestDist)
			{
				BestDist = MinPenetrationDist;
				HitNormal = -Axis * InvAxisMagnitude;
			}

			if (MaxPenetrationDist < BestDist)
			{
				BestDist = MaxPenetrationDist;
				HitNormal = Axis * InvAxisMagnitude;
			}
		}
	}

	return true;
}

bool FSeparatingAxisPointCheck::TestSeparatingAxisTriangle(const FVector& Axis)
{
	const float ProjectedV0 = FVector::DotProduct(Axis, PolyVertices[0]);
	const float ProjectedV1 = FVector::DotProduct(Axis, PolyVertices[1]);
	const float ProjectedV2 = FVector::DotProduct(Axis, PolyVertices[2]);
	const float ProjectedTriMin = FMath::Min3(ProjectedV0, ProjectedV1, ProjectedV2);
	const float ProjectedTriMax = FMath::Max3(ProjectedV0, ProjectedV1, ProjectedV2);

	return TestSeparatingAxisCommon(Axis, ProjectedTriMin, ProjectedTriMax);
}

bool FSeparatingAxisPointCheck::TestSeparatingAxisGeneric(const FVector& Axis)
{
	float ProjectedPolyMin = TNumericLimits<float>::Max();
	float ProjectedPolyMax = TNumericLimits<float>::Lowest();
	for (const auto& Vertex : PolyVertices)
	{
		const float ProjectedVertex = FVector::DotProduct(Axis, Vertex);
		ProjectedPolyMin = FMath::Min(ProjectedPolyMin, ProjectedVertex);
		ProjectedPolyMax = FMath::Max(ProjectedPolyMax, ProjectedVertex);
	}

	return TestSeparatingAxisCommon(Axis, ProjectedPolyMin, ProjectedPolyMax);
}

bool FSeparatingAxisPointCheck::FindSeparatingAxisTriangle()
{
	check(PolyVertices.Num() == 3);
	const FVector EdgeDir0 = PolyVertices[1] - PolyVertices[0];
	const FVector EdgeDir1 = PolyVertices[2] - PolyVertices[1];
	const FVector EdgeDir2 = PolyVertices[0] - PolyVertices[2];

	// Box Z edge x triangle edges.

	if (!TestSeparatingAxisTriangle(FVector(EdgeDir0.Y, -EdgeDir0.X, 0.0f)) ||
		!TestSeparatingAxisTriangle(FVector(EdgeDir1.Y, -EdgeDir1.X, 0.0f)) ||
		!TestSeparatingAxisTriangle(FVector(EdgeDir2.Y, -EdgeDir2.X, 0.0f)))
	{
		return false;
	}

	// Box Y edge x triangle edges.

	if (!TestSeparatingAxisTriangle(FVector(-EdgeDir0.Z, 0.0f, EdgeDir0.X)) ||
		!TestSeparatingAxisTriangle(FVector(-EdgeDir1.Z, 0.0f, EdgeDir1.X)) ||
		!TestSeparatingAxisTriangle(FVector(-EdgeDir2.Z, 0.0f, EdgeDir2.X)))
	{
		return false;
	}

	// Box X edge x triangle edges.

	if (!TestSeparatingAxisTriangle(FVector(0.0f, EdgeDir0.Z, -EdgeDir0.Y)) ||
		!TestSeparatingAxisTriangle(FVector(0.0f, EdgeDir1.Z, -EdgeDir1.Y)) ||
		!TestSeparatingAxisTriangle(FVector(0.0f, EdgeDir2.Z, -EdgeDir2.Y)))
	{
		return false;
	}

	// Box faces.

	if (!TestSeparatingAxisTriangle(FVector(0.0f, 0.0f, 1.0f)) ||
		!TestSeparatingAxisTriangle(FVector(1.0f, 0.0f, 0.0f)) ||
		!TestSeparatingAxisTriangle(FVector(0.0f, 1.0f, 0.0f)))
	{
		return false;
	}

	// Triangle normal.

	if (!TestSeparatingAxisTriangle(FVector::CrossProduct(EdgeDir1, EdgeDir0)))
	{
		return false;
	}

	return true;
}

bool FSeparatingAxisPointCheck::FindSeparatingAxisGeneric()
{
	check(PolyVertices.Num() > 3);
	int32 LastIndex = PolyVertices.Num() - 1;
	for (int32 Index = 0; Index < PolyVertices.Num(); Index++)
	{
		const FVector& V0 = PolyVertices[LastIndex];
		const FVector& V1 = PolyVertices[Index];
		const FVector EdgeDir = V1 - V0;

		// Box edges x polygon edge

		if (!TestSeparatingAxisGeneric(FVector(EdgeDir.Y, -EdgeDir.X, 0.0f)) ||
			!TestSeparatingAxisGeneric(FVector(-EdgeDir.Z, 0.0f, EdgeDir.X)) ||
			!TestSeparatingAxisGeneric(FVector(0.0f, EdgeDir.Z, -EdgeDir.Y)))
		{
			return false;
		}

		LastIndex = Index;
	}

	// Box faces.

	if (!TestSeparatingAxisGeneric(FVector(0.0f, 0.0f, 1.0f)) ||
		!TestSeparatingAxisGeneric(FVector(1.0f, 0.0f, 0.0f)) ||
		!TestSeparatingAxisGeneric(FVector(0.0f, 1.0f, 0.0f)))
	{
		return false;
	}

	// Polygon normal.

	int32 Index0 = PolyVertices.Num() - 2;
	int32 Index1 = Index0 + 1;
	for (int32 Index2 = 0; Index2 < PolyVertices.Num(); Index2++)
	{
		const FVector& V0 = PolyVertices[Index0];
		const FVector& V1 = PolyVertices[Index1];
		const FVector& V2 = PolyVertices[Index2];

		const FVector EdgeDir0 = V1 - V0;
		const FVector EdgeDir1 = V2 - V1;

		FVector Normal = FVector::CrossProduct(EdgeDir1, EdgeDir0);
		if (Normal.SizeSquared() > UE_SMALL_NUMBER)
		{
			if (!TestSeparatingAxisGeneric(Normal))
			{
				return false;
			}
			break;
		}

		Index0 = Index1;
		Index1 = Index2;
	}

	return true;
}

bool LineCheckWithTriangle(FHitResult& Result, const FVector& V1, const FVector& V2, const FVector& V3, const FVector& Start, const FVector& End, const FVector& Direction)
{
	FVector	Edge1 = V3 - V1,
		       Edge2 = V2 - V1,
		       P = Direction ^ Edge2;
	FVector::FReal	Determinant = Edge1 | P;

	if(Determinant < UE_DELTA)
	{
		return false;
	}

	FVector	T = Start - V1;
	FVector::FReal	U = T | P;

	if(U < 0.0f || U > Determinant)
	{
		return false;
	}

	FVector	Q = T ^ Edge1;
	FVector::FReal	V = Direction | Q;

	if(V < 0.0f || U + V > Determinant)
	{
		return false;
	}

	FVector::FReal	Time = (Edge2 | Q) / Determinant;

	if(Time < 0.0f || Time > Result.Time)
	{
		return false;
	}

	Result.Normal = ((V3-V2)^(V2-V1)).GetSafeNormal();
	Result.Time = static_cast<float>(((V1 - Start)|Result.Normal) / (Result.Normal|Direction));							// LWC_TODO: precision loss. Make FHitResult::Time/Distance doubles?

	return true;
}


#if !UE_BUILD_SHIPPING

DECLARE_LOG_CATEGORY_EXTERN(LogCollisionCommands, Log, All);
DEFINE_LOG_CATEGORY(LogCollisionCommands);

namespace CollisionResponseConsoleCommands
{
	static const TArray<FString> ResponseStrings = {TEXT("Ignore"), TEXT("Overlap"), TEXT("Block")};
	static const TArray<FString> ComplexityStrings = {TEXT("Default"), TEXT("SimpleAndComplex"), TEXT("UseSimpleAsComplex"), TEXT("UseComplexAsSimple")};

	FString GetCommaSeparatedList(const TArray<FString>& List)
	{
		FString Result;
		if (List.Num() > 0)
		{
			Result.Reserve(128);
			for (int32 i=0; i < List.Num()-1; i++)
			{
				Result.Append(List[i]);
				Result.Append(TEXT(", "));
			}
			Result.Append(List[List.Num()-1]);
		}
		return Result;
	}

	FString FillString(TCHAR Char, int32 Count)
	{
		FString Result;
		if (Count > 0)
		{
			Result.Reserve(Count);
			for (int32 i=0; i < Count; i++)
			{
				Result.AppendChar(TEXT('-'));
			}
		}
		return Result;
	}

	FString GetDisplayNameText(const UEnum* Enum, int64 Value, const FString& Fallback)
	{
		if (Enum)
		{
			#if WITH_EDITOR
				return Enum->GetDisplayNameTextByValue(Value).ToString();
			#else
				return FName::NameToDisplayString(Enum->GetNameStringByValue(Value), false);
			#endif
		}
		return Fallback;
	}

	FString FormatObjectName(UObject* Obj)
	{
		FString Result;
		if (Obj)
		{
			if (Obj->GetOuter())
			{
				Result = FString::Printf(TEXT("'%s'"), *Obj->GetPathName(Obj->GetOuter()->GetOuter()));
			}
			else
			{
				Result = FString::Printf(TEXT("'%s'"), *Obj->GetPathName(Obj->GetOuter()));
			}
		}
		return Result;
	}

	FString GetAssetName(const UPrimitiveComponent* Comp)
	{
		FString AssetName;
		if (const UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(Comp))
		{
			if (StaticMeshComp->GetStaticMesh())
			{
				AssetName = StaticMeshComp->GetStaticMesh()->GetPathName();
			}
		}
		else if (const USkinnedMeshComponent* SkinnedMeshComp = Cast<USkinnedMeshComponent>(Comp))
		{
			if (SkinnedMeshComp->GetSkinnedAsset())
			{
				AssetName = SkinnedMeshComp->GetSkinnedAsset()->GetPathName();
			}
		}

		return AssetName;
	}

	//////////////////////////////////////////////////////////////////////////
	// Helper to map enum to display name through a map (to avoid repeated string building).
	template<typename EnumType>
	const FString& MapEnumToDisplayName(const UEnum *ComplexityEnum, EnumType EnumFlag, TMap<EnumType, FString>& EnumToDisplayNameMap)
	{
		if (!EnumToDisplayNameMap.Contains(EnumFlag))
		{
			const FString ComplexityName = ComplexityEnum ? ComplexityEnum->GetNameByValue((int64)EnumFlag).ToString() : TEXT("<unknown>");
			const FString ComplexityDisplayName = GetDisplayNameText(ComplexityEnum, (int64)EnumFlag, ComplexityName);
			return EnumToDisplayNameMap.Add(EnumFlag, ComplexityDisplayName);
		}
		else
		{
			return EnumToDisplayNameMap.FindChecked(EnumFlag);
		}
	}

	void ListCollisionProfileNames()
	{
		TArray<TSharedPtr<FName>> ProfileNameList;
		UCollisionProfile::Get()->GetProfileNames(ProfileNameList);
		int32 Index = 0;
		for (TSharedPtr<FName> NamePtr : ProfileNameList)
		{
			const FName TemplateName = *NamePtr;
			UE_LOGF(LogCollisionCommands, Log, "%2d: %ls", Index++, *TemplateName.ToString());
		}
	}

	void ListCollisionChannelNames()
	{
		const UEnum *ChannelEnum = StaticEnum<ECollisionChannel>();
		if (ChannelEnum)
		{
			for (int32 ChannelValue = 0; ChannelValue < ECollisionChannel::ECC_MAX; ChannelValue++)
			{
				const FString ChannelShortName = ChannelEnum->GetNameStringByValue(ChannelValue);
				const FString ChannelDisplayName = GetDisplayNameText(ChannelEnum, ChannelValue, ChannelShortName);
				UE_LOGF(LogCollisionCommands, Log, "%2d: %ls (%ls)", ChannelValue, *ChannelShortName, *ChannelDisplayName);
			}
		}
	}

	struct FSortComponentsWithResponseToProfile
	{
		FSortComponentsWithResponseToProfile(ECollisionResponse InRequiredResponse)
		: RequiredResponse(InRequiredResponse)
		{
		}

		bool operator()(const UPrimitiveComponent& A, const UPrimitiveComponent& B) const
		{
			UObject* AOwner = A.GetOuter();
			UObject* BOwner = B.GetOuter();
			if (AOwner && BOwner)
			{
				// For overlaps, sort first by GetGenerateOverlapEvents()
				if (RequiredResponse == ECollisionResponse::ECR_Overlap)
				{
					if (A.GetGenerateOverlapEvents() != B.GetGenerateOverlapEvents())
					{
						return A.GetGenerateOverlapEvents();
					}
				}

				// Sort by name.
				return AOwner->GetName() < BOwner->GetName();
			}
			else
			{
				return false;
			}
		}

	private:
		ECollisionResponse RequiredResponse;
	};

	struct FSortComponentsForComplexity
	{
		FSortComponentsForComplexity(TMap<UPrimitiveComponent*, FString>& NameMap)
		: InternalNameMap(NameMap)
		{
		}

		bool operator()(const UPrimitiveComponent& A, const UPrimitiveComponent& B) const
		{
			// Sort by asset name;
			const FString NameA = InternalNameMap.FindRef(&A);
			const FString NameB = InternalNameMap.FindRef(&B);
			if (NameA != NameB)
			{
				return NameA < NameB;
			}

			// Sort by object name.
			UObject* AOwner = A.GetOuter();
			UObject* BOwner = B.GetOuter();
			if (AOwner && BOwner)
			{
				return AOwner->GetName() < BOwner->GetName();
			}
			else
			{
				return false;
			}
		}

	private: 

		TMap<UPrimitiveComponent*, FString>& InternalNameMap;
	};

	ECollisionResponse StringToCollisionResponse(const FString& InString)
	{
		int32 TestIndex = 0;
		for (const FString& CurrentString : ResponseStrings)
		{
			if (CurrentString == InString)
			{
				return ECollisionResponse(TestIndex);
			}
			TestIndex++;
		}

		return ECollisionResponse::ECR_MAX;
	}

	ECollisionChannel StringToCollisionChannel(const FString& InString)
	{
		const UEnum *ChannelEnum = StaticEnum<ECollisionChannel>();
		if (ChannelEnum)
		{
			int32 ChannelInt = INDEX_NONE;

			// Try the name they gave us.
			ChannelInt = ChannelEnum->GetValueByName(FName(*InString));
			if (ChannelInt != INDEX_NONE)
			{
				return ECollisionChannel(ChannelInt);
			}

			// Try with adding the prefix, ie "ECC_".
			const FString WithPrefixName = ChannelEnum->GenerateEnumPrefix() + TEXT("_") + InString;
			ChannelInt = ChannelEnum->GetValueByName(FName(*WithPrefixName));
			if (ChannelInt != INDEX_NONE)
			{
				return ECollisionChannel(ChannelInt);
			}

			// Try matching the display name
			const FString NullString;
			for (int32 ChannelIndex = 0; ChannelIndex < ChannelEnum->NumEnums(); ChannelIndex++)
			{
				int64 ChannelValue = ChannelEnum->GetValueByIndex(ChannelIndex);
				const FString ChannelDisplayName = GetDisplayNameText(ChannelEnum, ChannelValue, NullString);
				if (ChannelDisplayName == InString)
				{
					return ECollisionChannel(ChannelValue);
				}
			}
		}

		// Try parsing a digit, as in from the ListChannels command
		const int32 ChannelValue = FCString::IsNumeric(*InString) ? FCString::Atoi(*InString) : INDEX_NONE;
		return (ChannelValue >= 0 && ChannelValue < ECollisionChannel::ECC_MAX) ? ECollisionChannel(ChannelValue) : ECollisionChannel::ECC_MAX;
	}

	FName StringToCollisionProfile(const FString& InString)
	{
		const FName InStringAsName(*InString);
		FCollisionResponseTemplate Template;
		if (UCollisionProfile::Get()->GetProfileTemplate(InStringAsName, Template))
		{
			return InStringAsName;
		}

		if (FCString::IsNumeric(*InString))
		{
			const int32 ProfileIndex = FCString::Atoi(*InString);
			const FCollisionResponseTemplate* TemplateByIndex = UCollisionProfile::Get()->GetProfileByIndex(ProfileIndex);
			if (TemplateByIndex)
			{
				return TemplateByIndex->Name;
			}
		}

		return NAME_None;
	}

	ECollisionTraceFlag StringToCollisionComplexity(const FString& InString)
	{
		const UEnum *ComplexityEnum = StaticEnum<ECollisionTraceFlag>();
		if (ComplexityEnum)
		{
			int32 ComplexityInt = INDEX_NONE;

			// Try the name they gave us.
			ComplexityInt = ComplexityEnum->GetValueByName(FName(*InString));
			if (ComplexityInt != INDEX_NONE)
			{
				return ECollisionTraceFlag(ComplexityInt);
			}

			// Try with adding the prefix, ie "CTF_".
			const FString WithPrefixName = ComplexityEnum->GenerateEnumPrefix() + TEXT("_") + InString;
			ComplexityInt = ComplexityEnum->GetValueByName(FName(*WithPrefixName));
			if (ComplexityInt != INDEX_NONE)
			{
				return ECollisionTraceFlag(ComplexityInt);
			}

			// Try with adding the prefix and "Use", ie "CTF_Use" (for example 'Default'->'CTF_UseDefault').
			const FString WithPrefixUseName = ComplexityEnum->GenerateEnumPrefix() + TEXT("_Use") + InString;
			ComplexityInt = ComplexityEnum->GetValueByName(FName(*WithPrefixUseName));
			if (ComplexityInt != INDEX_NONE)
			{
				return ECollisionTraceFlag(ComplexityInt);
			}

			// Try matching the display name
			const FString NullString;
			for (int32 ComplexityIndex = 0; ComplexityIndex < ComplexityEnum->NumEnums(); ComplexityIndex++)
			{
				int64 ComplexityValue = ComplexityEnum->GetValueByIndex(ComplexityIndex);
				const FString ComplexityDisplayName = GetDisplayNameText(ComplexityEnum, ComplexityValue, NullString);
				if (ComplexityDisplayName == InString)
				{
					return ECollisionTraceFlag(ComplexityValue);
				}
			}
		}

		return CTF_MAX;
	}

	//////////////////////////////////////////////////////////////////////////
	void ListComponentsWithResponseToProfile(const ECollisionResponse RequiredResponse, const FName& ProfileToCheck)
	{
		// Look at all components and check response to profile
		TSet<UPrimitiveComponent*> Results;
		FCollisionResponseTemplate Template;
		if (UCollisionProfile::Get()->GetProfileTemplate(ProfileToCheck, Template))
		{
			for (TObjectIterator<UPrimitiveComponent> Iter(RF_NoFlags, /*bIncludeDerivedClasses*/ true, /*ExclusionFlags*/ EInternalObjectFlags::None); Iter; ++Iter)
			{
				UPrimitiveComponent* Comp = *Iter;
				
				// TEMP CRASH WORKAROUND: IsCollisionEnabled() fails on ULandscapeComponent CDO.
				if (Cast<ULandscapeHeightfieldCollisionComponent>(Comp))
				{
					continue;
				}

				if (IsQueryCollisionEnabled(Comp))
				{
					const ECollisionResponse CompResponse = Comp->GetCollisionResponseToChannel(Template.ObjectType.GetValue());
					const ECollisionResponse TemplateResponse = Template.ResponseToChannels.GetResponse(Comp->GetCollisionObjectType());
					const ECollisionResponse MinResponse = FMath::Min<ECollisionResponse>(CompResponse, TemplateResponse);
					if (MinResponse == RequiredResponse)
					{
						Results.Add(Comp);
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogCollisionCommands, Warning, "Could not find collision profile '%ls'. Use 'Collision.ListProfiles' to see a full list of available profiles.", *ProfileToCheck.ToString());
			return;
		}

		// Log results.
		if (Results.Num() > 0)
		{
			Results.Sort(FSortComponentsWithResponseToProfile(RequiredResponse));
			const UEnum *ChannelEnum = StaticEnum<ECollisionChannel>();
			TMap<ECollisionChannel, FString> EnumToDisplayNameMap;

			// Get max column widths for some data
			int32 MaxNameWidth = 0;
			int32 MaxChannelWidth = 0;
			int32 MaxProfileWidth = 0;
			for (UPrimitiveComponent* Comp : Results)
			{
				UObject* Outer = Comp->GetOuter();
				if (Outer)
				{
					const FString PathName = FormatObjectName(Comp);
					MaxNameWidth = FMath::Max<int32>(MaxNameWidth, PathName.Len());
				}

				if (ChannelEnum)
				{
					const FString& ChannelDisplayName = MapEnumToDisplayName<ECollisionChannel>(ChannelEnum, Comp->GetCollisionObjectType(), EnumToDisplayNameMap);
					MaxChannelWidth = FMath::Max<int32>(MaxChannelWidth, ChannelDisplayName.Len());
				}

				MaxProfileWidth = FMath::Max<int32>(MaxProfileWidth, Comp->GetCollisionProfileName().ToString().Len());
			}

			// Column headings
			FString Output;
			if (RequiredResponse == ECollisionResponse::ECR_Overlap)
			{
				Output = FString::Printf(TEXT("  #, GenerateEvents, %-*s, %-*s, %-*s, Path"), MaxNameWidth, TEXT("Component"), MaxChannelWidth, TEXT("ObjectType"), MaxProfileWidth, TEXT("Profile"));
			}
			else
			{
				Output = FString::Printf(TEXT("  #, %-*s, %-*s, %-*s, Path"), MaxNameWidth, TEXT("Component"), MaxChannelWidth, TEXT("ObjectType"), MaxProfileWidth, TEXT("Profile"));
			}
			UE_LOGF(LogCollisionCommands, Log, "%ls", *Output);
			const int32 TotalLen = Output.Len() + 16;
			const FString LineMarker = FillString(TCHAR('-'), TotalLen);
			UE_LOGF(LogCollisionCommands, Log, "%ls", *LineMarker);

			// Data
			int32 Index=0;
			for (UPrimitiveComponent* Comp : Results)
			{
				const FString& ChannelDisplayName = MapEnumToDisplayName<ECollisionChannel>(ChannelEnum, Comp->GetCollisionObjectType(), EnumToDisplayNameMap);
				UObject* Outer = Comp->GetOuter();
				if (Outer)
				{
					const FString OtherProfileName = Comp->GetCollisionProfileName().ToString();
					const FString PathName = FormatObjectName(Comp);
					if (RequiredResponse == ECollisionResponse::ECR_Overlap)
					{
						UE_LOGF(LogCollisionCommands, Log, "%3d, %-14ls, %-*ls, %-*ls, %-*ls, %ls",
							   Index, Comp->GetGenerateOverlapEvents() ? TEXT("true"):TEXT("false"), MaxNameWidth, *PathName, MaxChannelWidth, *ChannelDisplayName, MaxProfileWidth, *OtherProfileName, Outer->GetOuter() ? *GetPathNameSafe(Outer->GetOuter()) : *GetPathNameSafe(Outer));
					}
					else
					{
						UE_LOGF(LogCollisionCommands, Log, "%3d, %-*ls, %-*ls, %-*ls, %ls",
							   Index, MaxNameWidth, *PathName, MaxChannelWidth, *ChannelDisplayName, MaxProfileWidth, *OtherProfileName, Outer->GetOuter() ? *GetPathNameSafe(Outer->GetOuter()) : *GetPathNameSafe(Outer));
					}
					Index++;
				}
			}
			UE_LOGF(LogCollisionCommands, Log, "%ls", *LineMarker);
		}
		// Summary
		check(RequiredResponse < ECollisionResponse::ECR_MAX);
		UE_LOGF(LogCollisionCommands, Log, "Found %d components with '%ls' response to profile '%ls'.", Results.Num(), *ResponseStrings[(int32)RequiredResponse], *ProfileToCheck.ToString());
	}

	//////////////////////////////////////////////////////////////////////////
	// Args: <Response> <Profile>
	void Parse_ListComponentsWithResponseToProfile(const TArray<FString>& Args, UWorld* World)
	{
		if (Args.Num() < 2)
		{
			UE_LOGF(LogCollisionCommands, Warning, "Usage: 'Collision.ListComponentsWithResponseToProfile <Response> <Profile>'.");
			UE_LOGF(LogCollisionCommands, Warning, "  Response: %ls", *GetCommaSeparatedList(ResponseStrings));
			UE_LOGF(LogCollisionCommands, Warning, "  Profile:  Collision profile name or index. Use 'Collision.ListProfiles' to see a full list.");
			return;
		}

		// Arg0 : Response
		const FString& ResponseString = Args[0];
		ECollisionResponse RequiredResponse = StringToCollisionResponse(ResponseString);
		if (RequiredResponse == ECollisionResponse::ECR_MAX)
		{
			UE_LOGF(LogCollisionCommands, Warning, "Unknown response '%ls'. Must be one of %ls.", *ResponseString, *GetCommaSeparatedList(ResponseStrings));
			return;
		}

		// Arg1 : Profile
		const FString& ProfileNameString = Args[1];
		const FName ProfileToCheck = StringToCollisionProfile(ProfileNameString);
		if (ProfileToCheck == NAME_None)
		{
			UE_LOGF(LogCollisionCommands, Warning, "Could not find collision profile '%ls'. Use 'Collision.ListProfiles' to see a full list of available profiles.", *ProfileToCheck.ToString());
			return;
		}

		ListComponentsWithResponseToProfile(RequiredResponse, ProfileToCheck);
	}

	//////////////////////////////////////////////////////////////////////////
	void ListProfilesWithResponseToChannel(const ECollisionResponse RequiredResponse, const ECollisionChannel TestChannel)
	{
		// Match
		TSet<FName> Results;
		TArray<TSharedPtr<FName>> ProfileNameList;
		UCollisionProfile::Get()->GetProfileNames(ProfileNameList);
		int32 Index = 0;
		for (TSharedPtr<FName> NamePtr : ProfileNameList)
		{
			const FName TemplateName = *NamePtr;
			FCollisionResponseTemplate Template;
			if (UCollisionProfile::Get()->GetProfileTemplate(TemplateName, Template))
			{
				const ECollisionResponse TemplateResponse = Template.ResponseToChannels.GetResponse(TestChannel);
				if (TemplateResponse == RequiredResponse)
				{
					Results.Add(TemplateName);
				}
			}
			++Index;
		}

		// Display Data
		if (Results.Num() > 0)
		{
			Results.Sort(FNameLexicalLess());
			for (FName& ResultName : Results)
			{
				UE_LOGF(LogCollisionCommands, Log, "%ls", *ResultName.ToString());
			}
		}
		// Display Summary
		check(RequiredResponse < ECollisionResponse::ECR_MAX);
		const UEnum *ChannelEnum = StaticEnum<ECollisionChannel>();
		const FString ChannelName = (ChannelEnum ? ChannelEnum->GetNameStringByValue(TestChannel) : TEXT("<unknown>"));
		const FString ChannelDisplayName = GetDisplayNameText(ChannelEnum, TestChannel, ChannelName);
		UE_LOGF(LogCollisionCommands, Log, "----------------------------------------------------------------------");
		UE_LOGF(LogCollisionCommands, Log, "Found %d profiles with '%ls' response to channel '%ls' ('%ls')", Results.Num(), *ResponseStrings[(int32)RequiredResponse], *ChannelName, *ChannelDisplayName);
	}

	//////////////////////////////////////////////////////////////////////////
	// Args: <Response> <Channel>
	void Parse_ListProfilesWithResponseToChannel(const TArray<FString>& Args, UWorld* World)
	{
		if (Args.Num() < 2)
		{
			UE_LOGF(LogCollisionCommands, Warning, "Usage: 'Collision.ListProfilesWithResponseToChannel <Response> <Channel>'.");
			UE_LOGF(LogCollisionCommands, Warning, "  Response: %ls", *GetCommaSeparatedList(ResponseStrings));
			UE_LOGF(LogCollisionCommands, Warning, "  Profile:  Collision channel name or index. Use 'Collision.ListChannels' to see a full list.");
			return;
		}

		// Arg0 : Response
		const FString& ResponseString = Args[0];
		const ECollisionResponse RequiredResponse = StringToCollisionResponse(ResponseString);
		if (RequiredResponse == ECollisionResponse::ECR_MAX)
		{
			UE_LOGF(LogCollisionCommands, Warning, "Unknown response '%ls'. Must be one of %ls.", *ResponseString, *GetCommaSeparatedList(ResponseStrings));
			return;
		}

		// Arg1 : Channel
		const FString& ChannelNameString = Args[1];
		const ECollisionChannel Channel = StringToCollisionChannel(ChannelNameString);
		if (Channel == ECollisionChannel::ECC_MAX)
		{
			UE_LOGF(LogCollisionCommands, Warning, "Unknown channel '%ls. Use 'Collision.ListChannels' to see a full list.'", *ChannelNameString);
			return;
		}

		ListProfilesWithResponseToChannel(RequiredResponse, Channel);
	}


	//////////////////////////////////////////////////////////////////////////
	void ListObjectsWithCollisionComplexity(ECollisionTraceFlag Complexity)
	{
		if (Complexity == CTF_MAX)
		{
			return;
		}

		// See what "Default" means, so we can include those
		const ECollisionTraceFlag DefaultFlag = UPhysicsSettings::Get()->DefaultShapeComplexity;
		const ECollisionTraceFlag ActualComplexity = (Complexity == CTF_UseDefault) ? DefaultFlag : Complexity;

		// Look at all components and check collision complexity
		TSet<UPrimitiveComponent*> Results;
		for (TObjectIterator<UPrimitiveComponent> Iter(RF_NoFlags, /*bIncludeDerivedClasses*/ true, /*ExclusionFlags*/ EInternalObjectFlags::None); Iter; ++Iter)
		{
			UPrimitiveComponent* Comp = *Iter;
			if (Comp)
			{
				// Special case for UShapeComponent CDOs, GetBodySetup() asserts.
				if (Comp->GetClass() == UShapeComponent::StaticClass())
				{
					continue;
				}

				// Get collision complexity from body setup
				if (UBodySetup* BodySetup = Comp->GetBodySetup())
				{
					// If matching "Default", only list those explicitly set to default.
					if (Complexity == CTF_UseDefault && BodySetup->CollisionTraceFlag == CTF_UseDefault)
					{
						Results.Add(Comp);
					}
					else
					{
						// Using GetCollisionTraceFlag includes both the same requested complexity and those with Default if that is the same as the requested mode.
						const ECollisionTraceFlag BodyFlag = BodySetup->GetCollisionTraceFlag();
						if (Complexity == BodyFlag)
						{
							Results.Add(Comp);
						}
					}
				}
			}
		}

		// Log results.
		if (Results.Num() > 0)
		{
			// Fill component name Maps.
			TMap<UPrimitiveComponent*, FString> ComponentAssetNameMap;
			TMap<UPrimitiveComponent*, FString> ComponentPathNameMap;
			for (UPrimitiveComponent* Comp : Results)
			{
				const FString AssetName = GetAssetName(Comp);
				if (AssetName.Len() > 0)
				{
					ComponentAssetNameMap.Add(Comp, AssetName);
				}

				UObject* Outer = Comp->GetOuter();
				if (Outer)
				{
					const FString PathName = FormatObjectName(Comp);
					ComponentPathNameMap.Add(Comp, PathName);
				}
			}

			Results.Sort(FSortComponentsForComplexity(ComponentAssetNameMap));
			const UEnum *ComplexityEnum = StaticEnum<ECollisionTraceFlag>();
			if (!ensure(ComplexityEnum != nullptr))
			{
				return;
			}

			// Generate map of enum->name mappings
			TMap<ECollisionTraceFlag, FString> EnumToDisplayNameMap;

			// Get max column widths for some data
			const FString ComplexityHeading = "Collision Complexity";
			int32 MaxComplexityWidth = ComplexityHeading.Len();
			int32 MaxNameWidth = 0;
			int32 MaxAssetNameWidth = 0;
			for (UPrimitiveComponent* Comp : Results)
			{
				// Object name width
				UObject* Outer = Comp->GetOuter();
				if (Outer)
				{
					const FString PathName = ComponentPathNameMap.FindRef(Comp);
					MaxNameWidth = FMath::Max<int32>(MaxNameWidth, PathName.Len());
				}

				// Complexity display name width
				const UBodySetup* BodySetup = Comp->GetBodySetup();
				const ECollisionTraceFlag BodyFlag = BodySetup->CollisionTraceFlag;

				const FString& ComplexityDisplayName = MapEnumToDisplayName<ECollisionTraceFlag>(ComplexityEnum, BodyFlag, EnumToDisplayNameMap);				
				MaxComplexityWidth = FMath::Max(MaxComplexityWidth, ComplexityDisplayName.Len());

				// Asset name width
				const FString& AssetName = ComponentAssetNameMap.FindRef(Comp);
				MaxAssetNameWidth = FMath::Max(MaxAssetNameWidth, AssetName.Len());
			}

			// Display Column headings
			const FString Output = FString::Printf(TEXT("  #, %-*s, %-*s, %-*s, Path"), MaxNameWidth, TEXT("Component"), MaxComplexityWidth, *ComplexityHeading, MaxAssetNameWidth, TEXT("Asset"));
			UE_LOGF(LogCollisionCommands, Log, "%ls", *Output);
			const int32 TotalLen = Output.Len() + 16;
			const FString LineMarker = FillString(TCHAR('-'), TotalLen);
			UE_LOGF(LogCollisionCommands, Log, "%ls", *LineMarker);

			// Display Data
			int32 Index=0;
			for (UPrimitiveComponent* Comp : Results)
			{
				UObject* Outer = Comp->GetOuter();
				if (Outer)
				{
					const FString& PathName = ComponentPathNameMap.FindRef(Comp);
					const UBodySetup* BodySetup = Comp->GetBodySetup();
					const ECollisionTraceFlag BodyFlag = BodySetup->CollisionTraceFlag;
					const FString& ComplexityDisplayName = MapEnumToDisplayName<ECollisionTraceFlag>(ComplexityEnum, BodyFlag, EnumToDisplayNameMap);
					const FString& AssetName = ComponentAssetNameMap.FindRef(Comp);

					UE_LOGF(LogCollisionCommands, Log, "%3d, %-*ls, %-*ls, %-*ls, %ls",
						   Index, MaxNameWidth, *PathName, MaxComplexityWidth, *ComplexityDisplayName, MaxAssetNameWidth, *AssetName, Outer->GetOuter() ? *GetPathNameSafe(Outer->GetOuter()) : *GetPathNameSafe(Outer));
					Index++;
				}
			}
			UE_LOGF(LogCollisionCommands, Log, "%ls", *LineMarker);
		}
		// Display Summary
		UE_LOGF(LogCollisionCommands, Log, "Found %d components with '%ls' collision complexity.", Results.Num(), *ComplexityStrings[int32(Complexity)]);
	}



	//////////////////////////////////////////////////////////////////////////
	// Args: <Complexity>
	void Parse_ListObjectsWithCollisionComplexity(const TArray<FString>& Args, UWorld* World)
	{
		if (Args.Num() < 1)
		{
			UE_LOGF(LogCollisionCommands, Warning, "Usage: 'Collision.ListObjectsWithCollisionComplexity <Complexity>'.");
			UE_LOGF(LogCollisionCommands, Warning, "  Complexity: %ls", *GetCommaSeparatedList(ComplexityStrings));
			return;
		}

		// Arg0 : Complexity
		const FString& ComplexityString = Args[0];
		const ECollisionTraceFlag Complexity = StringToCollisionComplexity(ComplexityString);
		if (Complexity == CTF_MAX)
		{
			UE_LOGF(LogCollisionCommands, Warning, "Unknown complexity '%ls'. Must be one of %ls.", *ComplexityString, *GetCommaSeparatedList(ComplexityStrings));
			return;
		}

		ListObjectsWithCollisionComplexity(Complexity);
	}


	//////////////////////////////////////////////////////////////////////////
	// Console commands

	static FAutoConsoleCommand ListProfilesCommand(
		TEXT("Collision.ListProfiles"),
		TEXT("ListProfiles"),
		FConsoleCommandDelegate::CreateStatic(ListCollisionProfileNames)
	);

	static FAutoConsoleCommand ListChannelsCommand(
		TEXT("Collision.ListChannels"),
		TEXT("ListChannels"),
		FConsoleCommandDelegate::CreateStatic(ListCollisionChannelNames)
	);

	static FAutoConsoleCommandWithWorldAndArgs ListComponentsWithResponseToProfileCommand(
		TEXT("Collision.ListComponentsWithResponseToProfile"),
		TEXT(""),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(CollisionResponseConsoleCommands::Parse_ListComponentsWithResponseToProfile)
	);

	static FAutoConsoleCommandWithWorldAndArgs ListProfilesWithResponseToChannelCommand(
		TEXT("Collision.ListProfilesWithResponseToChannel"),
		TEXT(""),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(CollisionResponseConsoleCommands::Parse_ListProfilesWithResponseToChannel)
	);

	static FAutoConsoleCommandWithWorldAndArgs ListProfilesWithCollisionComplexityCommand(
		TEXT("Collision.ListObjectsWithCollisionComplexity"),
		TEXT(""),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(CollisionResponseConsoleCommands::Parse_ListObjectsWithCollisionComplexity)
	);

}

#endif //!UE_BUILD_SHIPPING




