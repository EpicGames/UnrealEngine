// Copyright Epic Games, Inc. All Rights Reserved.

#include "Collision/CollisionConversions.h"
#include "BodySetupCore.h"
#include "Chaos/ChaosUserEntity.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "Chaos/Capsule.h"
#include "Chaos/ImplicitObjectType.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "Components/PrimitiveComponent.h"

#include "EngineLogs.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"

#include "Physics/Experimental/ChaosInterfaceWrapper.h"

#include "PhysicsEngine/CollisionQueryFilterCallback.h"
#include "Engine/ActorInstanceManagerInterface.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"

int32 GetItemIndex(const FOverlapResult& InResult)
{
	// This causes different behavior when enabled as we don't deduplicate hit results when `OwnerComponent->bMultiBodyOverlap` is false.
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return InResult.GetItemIndexInternal();
#else
	return InResult.GetItemIndex();
#endif
}

// Used to place overlaps into a TMap when deduplicating them
struct FOverlapKey
{
	const Chaos::FPhysicsObjectHandle PhysicsObject;
	TWeakObjectPtr<class UPrimitiveComponent> Component;
	const int32 ComponentIndex;

	FOverlapKey(const FOverlapResult& InResult)
		: PhysicsObject(InResult.PhysicsObject)
		, Component(InResult.Component)
		, ComponentIndex(GetItemIndex(InResult))
	{
	}

	friend bool operator==(const FOverlapKey& X, const FOverlapKey& Y)
	{
		return  (X.PhysicsObject == Y.PhysicsObject) && (X.ComponentIndex == Y.ComponentIndex) && (X.Component == Y.Component);
	}
};

uint32 GetTypeHash(const FOverlapKey& Key)
{
	return GetTypeHash(Key.PhysicsObject) ^ GetTypeHash(Key.Component) ^ GetTypeHash(Key.ComponentIndex);
}

extern int32 CVarShowInitialOverlaps;

// Forward declare, I don't want to move the entire function right now or we lose change history.
template <typename THitLocation>
static bool ConvertOverlappedShapeToImpactHit(const UWorld* World, const THitLocation& Hit, const FVector& StartLoc, const FVector& EndLoc, FHitResult& OutResult, const FPhysicsGeometry& Geom, const FTransform& QueryTM, const Chaos::Filter::FQueryFilterData& QueryFilter, bool bReturnPhysMat);

DECLARE_CYCLE_STAT(TEXT("ConvertQueryHit"), STAT_ConvertQueryImpactHit, STATGROUP_Collision);
DECLARE_CYCLE_STAT(TEXT("ConvertOverlapToHit"), STAT_CollisionConvertOverlapToHit, STATGROUP_Collision);
DECLARE_CYCLE_STAT(TEXT("ConvertOverlap"), STAT_CollisionConvertOverlap, STATGROUP_Collision);
DECLARE_CYCLE_STAT(TEXT("SetHitResultFromShapeAndFaceIndex"), STAT_CollisionSetHitResultFromShapeAndFaceIndex, STATGROUP_Collision);

#define ENABLE_CHECK_HIT_NORMAL  (!(UE_BUILD_SHIPPING || UE_BUILD_TEST))

#if ENABLE_CHECK_HIT_NORMAL
/* Validate Normal of OutResult. We're on hunt for invalid normal */
static void CheckHitResultNormal(const FHitResult& OutResult, const TCHAR* Message, const FVector& Start=FVector::ZeroVector, const FVector& End = FVector::ZeroVector, const FPhysicsGeometry* Geom=nullptr, const FVector& NormalPreSafeNormalize = FVector::ZeroVector)
{
	using namespace ChaosInterface;

	if(!OutResult.bStartPenetrating && !OutResult.Normal.IsNormalized())
	{
		UE_LOGF(LogPhysics, Warning, "(%ls) Non-normalized OutResult.Normal from hit conversion: %ls (Component- %ls)", Message, *OutResult.Normal.ToString(), *OutResult.Component->GetFullName());
		UE_LOGF(LogPhysics, Warning, "Start Loc(%ls), End Loc(%ls), Hit Loc(%ls), ImpactNormal(%ls) NormalPreSafeNormalize(%ls)", *Start.ToString(), *End.ToString(), *OutResult.Location.ToString(), *OutResult.ImpactNormal.ToString(), *NormalPreSafeNormalize.ToString() );
		if (Geom)
		{
			if (GetType(*Geom) == ECollisionShapeType::Capsule)
			{
				const FPhysicsCapsuleGeometry& Capsule = (FPhysicsCapsuleGeometry&)*Geom;
				UE_LOGF(LogPhysics, Warning, "Capsule radius (%f), Capsule Halfheight (%f)", GetRadius(Capsule), GetHalfHeight(Capsule));
			}
		}
		ensure(OutResult.Normal.IsNormalized());
	}
}
#endif // ENABLE_CHECK_HIT_NORMAL




static FVector FindSimpleOpposingNormal(const FHitLocation& Hit, const FVector& TraceDirectionDenorm, const FVector InNormal)
{
	// We don't compute anything special
	return InNormal;
}


/**
 * Util to find the normal of the face that we hit. Will use faceIndex from the hit if possible.
 * @param PHit - incoming hit from Physics
 * @param TraceDirectionDenorm - direction of sweep test (not normalized)
 * @param InNormal - default value in case no new normal is computed.
 * @return New normal we compute for geometry.
 */
template <typename THitLocation>
static FVector FindGeomOpposingNormal(ECollisionShapeType QueryGeomType, const THitLocation& Hit, const FVector& TraceDirectionDenorm, const FVector InNormal)
{
	// TODO: can we support other shapes here as well?
	if (QueryGeomType == ECollisionShapeType::Capsule || QueryGeomType == ECollisionShapeType::Sphere)
	{
		if (const FPhysicsShape* Shape = GetShape(Hit))
		{
			const Chaos::EImplicitObjectType ShapeType = Shape->GetGeometry()->GetNestedType();
			if (ShapeType == Chaos::ImplicitObjectType::Union && !Hit.FaceNormal.IsNearlyZero())
			{
				return Hit.FaceNormal;
			}
			else
			{
				const FTransform ActorTM(Hit.Actor->GetR(), Hit.Actor->GetX());
				const FVector LocalInNormal = ActorTM.InverseTransformVectorNoScale(InNormal);
				const FVector LocalTraceDirectionDenorm = ActorTM.InverseTransformVectorNoScale(TraceDirectionDenorm);
				const FVector LocalNormal = Shape->GetGeometry()->FindGeometryOpposingNormal(LocalTraceDirectionDenorm, Hit.FaceIndex, LocalInNormal);
				return ActorTM.TransformVectorNoScale(LocalNormal);
			}
		}
	}

	return InNormal;
}

/** Set info in the HitResult (Actor, Component, PhysMaterial, BoneName, Item) based on the supplied shape and face index */
static void SetHitResultFromShapeAndFaceIndex(const FPhysicsShape& Shape,  const FPhysicsActor& Actor, const uint32 FaceIndex, const FVector& HitLocation, FHitResult& OutResult, bool bReturnPhysMat)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_CollisionSetHitResultFromShapeAndFaceIndex);

	const int32 ShapeIndex = Shape.GetShapeIndex();
	CHAOS_CHECK(ShapeIndex < (int32)TNumericLimits<uint8>::Max()); // I could just write < 256, but this makes it more clear *why*
	OutResult.ElementIndex = (uint8)ShapeIndex;

	TWeakObjectPtr<UPrimitiveComponent> OwningComponent;
	OutResult.PhysicsObject = nullptr;
	
	if (!HasValidUserData(Actor))
	{
		return;
	}
	else if (const FBodyInstance* BodyInst = GetUserData(Actor))
	{
		BodyInst = FPhysicsInterface::ShapeToOriginalBodyInstance(BodyInst, &Shape);

		//Normal case where we hit a body
		OutResult.Item = BodyInst->InstanceBodyIndex;
		const UBodySetupCore* BodySetup = BodyInst->BodySetup.Get(); //this data should be immutable at runtime so ok to check from worker thread.
		if (BodySetup)
		{
			OutResult.BoneName = BodySetup->BoneName;
		}

		OwningComponent = BodyInst->OwnerComponent;

		// Use GetActorReferenceWithWelding() instead of GetPhysicsActor() so that
		// welded children resolve to their parent's PhysicsObject rather than returning null.
		// When a body is welded, its shapes are merged into the parent particle,
		// so the parent's PhysicsObject is the correct one for force/impulse application.
		const FPhysicsActorHandle& PhysicsActor = BodyInst->GetActorReferenceWithWelding();
		OutResult.PhysicsObject = PhysicsActor ? PhysicsActor->GetPhysicsObject() : nullptr;
	}
	else if (Chaos::FPhysicsObject* PhysObject = GetPhysicsObjectFromUserData(Actor))
	{
		OutResult.PhysicsObject = PhysObject;
		OutResult.Item = Chaos::FPhysicsObjectInterface::GetId(PhysObject);
		OutResult.BoneName = Chaos::FPhysicsObjectInterface::GetName(PhysObject);
	}
	else
	{
		// Currently geom collections are registered with a primitive component user data, but maybe custom should be adapted
		// to be more general so we can support leaf identification #BGTODO
		UPrimitiveComponent* PossibleOwner = GetPrimitiveComponentFromUserData(Actor);

		if(PossibleOwner)
		{
			OwningComponent = PossibleOwner;
			OutResult.Item = INDEX_NONE;
			OutResult.BoneName = NAME_None;

			// if we have a geometry component let's extract the index of the active piece we have a hit for
			if (const IPhysicsProxyBase* ActorProxy = Actor.GetProxy())
			{
				if (ActorProxy->GetType() == EPhysicsProxyType::GeometryCollectionType)
				{
					const FGeometryCollectionPhysicsProxy* ConcreteProxy = static_cast<const FGeometryCollectionPhysicsProxy*>(ActorProxy);
					const FGeometryCollectionItemIndex ItemIndex = ConcreteProxy->GetItemIndexFromGTParticle_External(Actor.CastToRigidParticle());
					OutResult.Item = ItemIndex.GetItemIndex();
					OutResult.BoneName = ConcreteProxy->GetTransformName_External(ItemIndex);
					OutResult.PhysicsObject = PossibleOwner->GetPhysicsObjectById(OutResult.Item);
				}
			}
		}
		else
		{
			ensureMsgf(false, TEXT("SetHitResultFromShapeAndFaceIndex hit shape with invalid userData"));
		}
	}

	if (OutResult.PhysicsObject != nullptr)
	{
		OutResult.PhysicsObjectOwner = OwningComponent;
		if (FChaosUserEntityAppend* ChaosUserEntityAppend = FChaosUserData::Get<FChaosUserEntityAppend>(Actor.UserData()))
		{
			OutResult.PhysicsObjectOwner = ChaosUserEntityAppend->GetOwnerObject();
		}
	}

	OutResult.PhysMaterial = nullptr;

	// Grab actor/component
	if (!OwningComponent.IsExplicitlyNull())
	{
		OutResult.Component = OwningComponent;

		// Create a handle that needs to be resolved later when accessed in game thread,
		// since resolving FActorInstanceHandle may require to access related UObjects (e.g. owner actor), which is not safe outside game thread.
		OutResult.HitObjectHandle = FActorInstanceHandle::MakeActorHandleToResolve(OwningComponent, OutResult.Item);
	}

	if (bReturnPhysMat)
	{
		if (const FPhysicsMaterial* PhysicsMaterial = GetMaterialFromInternalFaceIndexAndHitLocation(Shape, Actor, FaceIndex, HitLocation))
		{
			OutResult.PhysMaterial = GetUserData(*PhysicsMaterial);
		}
	}

	OutResult.FaceIndex = INDEX_NONE;
}

const FPhysicsActor* GetGTActor(const FPhysicsActor* GTActor) { return GTActor; }

template <bool bGT>
const FPhysicsShape* GetGTShape(const FPhysicsShape* GTShape, const FPhysicsActor* GTActor) { return GTShape; }

template <>
const FPhysicsShape* GetGTShape<false>(const FPhysicsShape* PTShape, const FPhysicsActor* GTActor)
{
	//This assumes that the geometry hasn't changed.
	//If it has you will either get null, or the wrong GT shape (note this gt shape is valid and safe to access, user will just get incorrect result)
	//This is expected because you are accessing GT timeline data from the PT timeline
	if (GTActor)
	{
		const Chaos::FShapesArray& Shapes = GTActor->ShapesArray();
		const int32 Idx = PTShape->GetShapeIndex();
		if (Idx < Shapes.Num())
		{
			return Shapes[Idx].Get();
		}
	}

	return nullptr;
}

const FPhysicsActor* GetGTActor(const Chaos::FGeometryParticleHandle* PTActor)
{
	const IPhysicsProxyBase* Proxy = PTActor->PhysicsProxy();
	if (Proxy)
	{
		const EPhysicsProxyType ProxyType = Proxy->GetType();
		switch (ProxyType)
		{
		case EPhysicsProxyType::ClusterUnionProxy:
			return static_cast<const Chaos::FClusterUnionPhysicsProxy*>(Proxy)->GetParticle_External();
		case EPhysicsProxyType::GeometryCollectionType:
			return static_cast<const FGeometryCollectionPhysicsProxy*>(Proxy)->GetInitialRootParticle_External();
		case EPhysicsProxyType::SingleParticleProxy:
			return static_cast<const Chaos::FSingleParticlePhysicsProxy*>(Proxy)->GetParticle_LowLevel();
		default:
			break;
		}
	}

	return nullptr;
}

template <typename THitLocation>
EConvertQueryResult ConvertQueryImpactHitImp(const UWorld* World, const THitLocation& Hit, FHitResult& OutResult, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, bool bReturnFaceIndex, bool bReturnPhysMat)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_ConvertQueryImpactHit);

#if WITH_EDITOR
	if(bReturnFaceIndex && World && World->IsGameWorld())
	{
		if(UPhysicsSettings::Get()->bSuppressFaceRemapTable)
		{
			bReturnFaceIndex = false;	//The editor uses the remap table, so we modify this to get the same results as you would in a cooked build
		}
	}
#endif

	FHitFlags Flags = GetFlags(Hit);
	checkSlow(Flags & EHitFlags::Distance);

	const FPhysicsShape* pHitShape = GetShape(Hit);
	const auto pHitActor = GetActor(Hit);

	const uint32 InternalFaceIndex = GetInternalFaceIndex(Hit);
	const bool bInitialOverlap = HadInitialOverlap(Hit);

	if (bInitialOverlap && Geom)
	{
		ConvertOverlappedShapeToImpactHit(World, Hit, StartLoc, EndLoc, OutResult, *Geom, QueryTM, QueryFilter, bReturnPhysMat);

		if(pHitShape)
		{
			const ECollisionShapeType GeometryType = GetGeometryType(*pHitShape);
			const bool bTriMesh = GeometryType == ECollisionShapeType::Trimesh;
			const bool bHeightField = GeometryType == ECollisionShapeType::Heightfield;
			const bool bValidInternalFace = InternalFaceIndex != GetInvalidPhysicsFaceIndex();
			if(bReturnFaceIndex && bValidInternalFace)
			{
				if (bTriMesh)
				{
					OutResult.FaceIndex = GetTriangleMeshExternalFaceIndex(*pHitShape, InternalFaceIndex);
				}
				else if (bHeightField)
				{
					OutResult.FaceIndex = InternalFaceIndex;
				}
			}
		}

		return EConvertQueryResult::Valid;
	}

	if ((pHitShape == nullptr) || (pHitActor == nullptr))
	{
		OutResult.Reset();
		return EConvertQueryResult::Invalid;
	}

	const FPhysicsShape& HitShape = *pHitShape;
	const auto& HitActor = *pHitActor;

	//Get the GT actor/shape for accessing GT specific data, should not be used for things like hit location
	const FPhysicsActor* GTActor = GetGTActor(pHitActor);
	//Note that a non-gt query could still happen on gt if we're in a fixed tick
	constexpr bool bIsGTQuery = std::is_same<THitLocation, FHitLocation>::value;
	const FPhysicsShape* GTShape = GetGTShape<bIsGTQuery>(pHitShape, GTActor);

	// See if this is a 'blocking' hit
	const Chaos::Filter::FShapeFilterData ShapeFilter = GetShapeFilterData(HitShape);
	const ECollisionQueryHitType HitType = FCollisionQueryFilterCallback::CalcQueryHitType(QueryFilter, ShapeFilter);
	OutResult.bBlockingHit = (HitType == ECollisionQueryHitType::Block);
	OutResult.bStartPenetrating = bInitialOverlap;

	// calculate the hit time
	const float HitTime = GetDistance(Hit)/CheckLength;
	OutResult.Time = HitTime;
	OutResult.Distance = GetDistance(Hit);

	// figure out where the the "safe" location for this shape is by moving from the startLoc toward the ImpactPoint
	const FVector TraceStartToEnd = EndLoc - StartLoc;
	const FVector SafeLocationToFitShape = StartLoc + (HitTime * TraceStartToEnd);
	OutResult.Location = SafeLocationToFitShape;

	const bool bUseReturnedPoint = ((Flags & EHitFlags::Position) && !bInitialOverlap);
	FVector Position = StartLoc;
	if (bUseReturnedPoint)
	{
		Position = GetPosition(Hit);
		if (Position.ContainsNaN() && GTActor && GTShape)
		{
#if ENABLE_NAN_DIAGNOSTIC
			SetHitResultFromShapeAndFaceIndex(*GTShape, *GTActor, InternalFaceIndex, OutResult.ImpactPoint, OutResult, bReturnPhysMat);
			UE_LOGF(LogCore, Error, "ConvertQueryImpactHit() NaN details:\n>> Actor:%ls (%ls)\n>> Component:%ls\n>> Item:%d\n>> BoneName:%ls\n>> Time:%f\n>> Distance:%f\n>> Location:%ls\n>> bIsBlocking:%d\n>> bStartPenetrating:%d",
				*OutResult.GetHitObjectHandle().GetName(), OutResult.HasValidHitObjectHandle() ? *OutResult.GetHitObjectHandle().FetchActor()->GetPathName() : TEXT("no path"),
				*GetNameSafe(OutResult.GetComponent()), OutResult.Item, *OutResult.BoneName.ToString(),
				OutResult.Time, OutResult.Distance, *OutResult.Location.ToString(), OutResult.bBlockingHit ? 1 : 0, OutResult.bStartPenetrating ? 1 : 0);
#endif // ENABLE_NAN_DIAGNOSTIC

			OutResult.Reset();
			logOrEnsureNanError(TEXT("ConvertQueryImpactHit() received NaN/Inf for position: %s"), *Position.ToString());
			return EConvertQueryResult::Invalid;
		}
	}
	OutResult.ImpactPoint = Position;
	
	// Caution: we may still have an initial overlap, but with null Geom. This is the case for RayCast results.
	const bool bUseReturnedNormal = ((Flags & EHitFlags::Normal) && !bInitialOverlap);
	const FVector HitNormal = GetNormal(Hit);
	if (bUseReturnedNormal && HitNormal.ContainsNaN() && GTActor && GTShape)
	{
#if ENABLE_NAN_DIAGNOSTIC
		SetHitResultFromShapeAndFaceIndex(*GTShape, *GTActor, InternalFaceIndex, OutResult.ImpactPoint, OutResult, bReturnPhysMat);
		UE_LOGF(LogCore, Error, "ConvertQueryImpactHit() NaN details:\n>> Actor:%ls (%ls)\n>> Component:%ls\n>> Item:%d\n>> BoneName:%ls\n>> Time:%f\n>> Distance:%f\n>> Location:%ls\n>> bIsBlocking:%d\n>> bStartPenetrating:%d",
			*OutResult.GetHitObjectHandle().GetName(), OutResult.HasValidHitObjectHandle() ? *OutResult.GetHitObjectHandle().FetchActor()->GetPathName() : TEXT("no path"),
			*GetNameSafe(OutResult.GetComponent()), OutResult.Item, *OutResult.BoneName.ToString(),
			OutResult.Time, OutResult.Distance, *OutResult.Location.ToString(), OutResult.bBlockingHit ? 1 : 0, OutResult.bStartPenetrating ? 1 : 0);
#endif // ENABLE_NAN_DIAGNOSTIC

		OutResult.Reset();
		logOrEnsureNanError(TEXT("ConvertQueryImpactHit() received NaN/Inf for normal: %s"), *HitNormal.ToString());
		return EConvertQueryResult::Invalid;
	}

	FVector OriginalNormal = bUseReturnedNormal ? HitNormal : -TraceStartToEnd;
	FVector Normal = OriginalNormal.GetSafeNormal();
	OutResult.Normal = Normal;
	OutResult.ImpactNormal = Normal;

	OutResult.TraceStart = StartLoc;
	OutResult.TraceEnd = EndLoc;

	// Fill in Actor, Component, material, etc.
	if(GTActor && GTShape)
	{
		SetHitResultFromShapeAndFaceIndex(*GTShape, *GTActor, InternalFaceIndex, OutResult.ImpactPoint, OutResult, bReturnPhysMat);
	}

#if ENABLE_CHECK_HIT_NORMAL
	CheckHitResultNormal(OutResult, TEXT("Invalid Normal from ConvertQueryImpactHit"), StartLoc, EndLoc, Geom, OriginalNormal);
#endif // ENABLE_CHECK_HIT_NORMAL

	if (bUseReturnedNormal && !Normal.IsNormalized())
	{
		// TraceStartToEnd should never be zero, because of the length restriction in the raycast and sweep tests.
		Normal = -TraceStartToEnd.GetSafeNormal();
		OutResult.Normal = Normal;
		OutResult.ImpactNormal = Normal;
	}

	const ECollisionShapeType SweptGeometryType = Geom ? GetType(*Geom) : ECollisionShapeType::None;
	OutResult.ImpactNormal = FindGeomOpposingNormal(SweptGeometryType, Hit, TraceStartToEnd, Normal);

	ECollisionShapeType GeomType = GetGeometryType(HitShape);

	if(GeomType == ECollisionShapeType::Heightfield)
	{
		if (InternalFaceIndex != GetInvalidPhysicsFaceIndex())
		{
			// Lookup physical material for heightfields
			if (bReturnPhysMat && GTActor && GTShape)
			{
				if (const FPhysicsMaterial* Material = GetMaterialFromInternalFaceIndex(*GTShape, *GTActor, InternalFaceIndex))
				{
					OutResult.PhysMaterial = GetUserData(*Material);
				}
			}
			if (bReturnFaceIndex)
			{
				OutResult.FaceIndex = InternalFaceIndex;
			}
		}
	}
	else if (bReturnFaceIndex && GeomType == ECollisionShapeType::Trimesh && InternalFaceIndex != GetInvalidPhysicsFaceIndex() && GTShape)
	{
		OutResult.FaceIndex = GetTriangleMeshExternalFaceIndex(*GTShape, InternalFaceIndex);
	}
	else if (GeomType == ECollisionShapeType::Convex)
	{
		// Convex gets a face index set so that an impact normal can be set for sweeps.
		// This isn't a true face index that matches to anything a user would expect, so clear this away.
		OutResult.FaceIndex = INDEX_NONE;
	}

	return EConvertQueryResult::Valid;
}

EConvertQueryResult ConvertQueryImpactHit(const UWorld* World, const FHitLocation& Hit, FHitResult& OutResult, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, bool bReturnFaceIndex, bool bReturnPhysMat)
{
	return ConvertQueryImpactHitImp(World, Hit, OutResult, CheckLength, QueryFilter, StartLoc, EndLoc, Geom, QueryTM, bReturnFaceIndex, bReturnPhysMat);
}
EConvertQueryResult ConvertQueryImpactHit(const UWorld* World, const ChaosInterface::FPTLocationHit& Hit, FHitResult& OutResult, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, bool bReturnFaceIndex, bool bReturnPhysMat)
{
	return ConvertQueryImpactHitImp(World, Hit, OutResult, CheckLength, QueryFilter, StartLoc, EndLoc, Geom, QueryTM, bReturnFaceIndex, bReturnPhysMat);
}

template <typename HitType>
EConvertQueryResult ConvertTraceResults(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, HitType* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FHitResult>& OutHits, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat)
{
	OutHits.Reserve(OutHits.Num() + NumHits);
	EConvertQueryResult ConvertResult = EConvertQueryResult::Valid;
	bool bHadBlockingHit = false;
	const FVector Dir = (EndLoc - StartLoc).GetSafeNormal();

	for(int32 i=0; i<NumHits; i++)
	{
		HitType& Hit = Hits[i];
		if(GetDistance(Hit) <= MaxDistance)
		{
			if constexpr (std::is_same_v<HitType, FHitSweep> || std::is_same_v<HitType, ChaosInterface::FPTSweepHit>)
			{
				if (!HadInitialOverlap(Hit))
				{
					SetInternalFaceIndex(Hit, FindFaceIndex(Hit, Dir));
				}
			}

			FHitResult& NewResult = OutHits[OutHits.AddDefaulted()];
			if (ConvertQueryImpactHit(World, Hit, NewResult, CheckLength, QueryFilter, StartLoc, EndLoc, Geom, QueryTM, bReturnFaceIndex, bReturnPhysMat) == EConvertQueryResult::Valid)
			{
				bHadBlockingHit |= NewResult.bBlockingHit;
			}
			else
			{
				// Reject invalid result (this should be rare). Remove from the results.
				OutHits.Pop(EAllowShrinking::No);
				ConvertResult = EConvertQueryResult::Invalid;
			}
			
		}
	}

	// Sort results from first to last hit
	OutHits.Sort( FCompareFHitResultTime() );
	OutHasValidBlockingHit = bHadBlockingHit;
	return ConvertResult;
}

template <typename Hit>
EConvertQueryResult ConvertTraceResults(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, Hit* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, FHitResult& OutHit, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat)
{
	const FVector Dir = (EndLoc - StartLoc).GetSafeNormal();
	if constexpr (std::is_same_v<Hit, FHitSweep> || std::is_same_v<Hit, ChaosInterface::FPTSweepHit>)
	{
		if (!HadInitialOverlap(Hits[0]))
		{
			SetInternalFaceIndex(Hits[0], FindFaceIndex(Hits[0], Dir));
		}
	}
	EConvertQueryResult Result = ConvertQueryImpactHit(World, Hits[0], OutHit, CheckLength, QueryFilter, StartLoc, EndLoc, Geom, QueryTM, bReturnFaceIndex, bReturnPhysMat);
	OutHasValidBlockingHit = Result == EConvertQueryResult::Valid;
	return Result;
}

template EConvertQueryResult ConvertTraceResults<FHitSweep>(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, FHitSweep* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FHitResult>& OutHits, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);
template EConvertQueryResult ConvertTraceResults<FHitSweep>(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, FHitSweep* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, FHitResult& OutHit, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);
template EConvertQueryResult ConvertTraceResults<ChaosInterface::FPTSweepHit>(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, ChaosInterface::FPTSweepHit* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FHitResult>& OutHits, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);
template EConvertQueryResult ConvertTraceResults<ChaosInterface::FPTSweepHit>(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, ChaosInterface::FPTSweepHit* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, FHitResult& OutHit, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);
template EConvertQueryResult ConvertTraceResults<FHitRaycast>(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, FHitRaycast* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FHitResult>& OutHits, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);
template EConvertQueryResult ConvertTraceResults<FHitRaycast>(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, FHitRaycast* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, FHitResult& OutHit, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);
template EConvertQueryResult ConvertTraceResults<ChaosInterface::FPTRaycastHit>(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, ChaosInterface::FPTRaycastHit* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FHitResult>& OutHits, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);
template EConvertQueryResult ConvertTraceResults<ChaosInterface::FPTRaycastHit>(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, ChaosInterface::FPTRaycastHit* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, FHitResult& OutHit, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);

/** Util to convert an overlapped shape into a sweep hit result, returns whether it was a blocking hit. */
template <typename THitLocation>
static bool ConvertOverlappedShapeToImpactHit(const UWorld* World, const THitLocation& Hit, const FVector& StartLoc, const FVector& EndLoc, FHitResult& OutResult, const FPhysicsGeometry& Geom, const FTransform& QueryTM, const Chaos::Filter::FQueryFilterData& QueryFilter, bool bReturnPhysMat)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_CollisionConvertOverlapToHit);

	const FPhysicsShape* pHitShape = GetShape(Hit);
	const auto pHitActor = GetActor(Hit);
	if ((pHitShape == nullptr) || (pHitActor == nullptr))
	{
		OutResult.Reset();
		return false;
	}

	const FPhysicsShape& HitShape = *pHitShape;
	const auto& HitActor = *pHitActor;

	// See if this is a 'blocking' hit
	const Chaos::Filter::FShapeFilterData ShapeFilter = GetShapeFilterData(HitShape);
	const ECollisionQueryHitType HitType = FCollisionQueryFilterCallback::CalcQueryHitType(QueryFilter, ShapeFilter);
	const bool bBlockingHit = (HitType == ECollisionQueryHitType::Block);
	OutResult.bBlockingHit = bBlockingHit;

	// Time of zero because initially overlapping
	OutResult.bStartPenetrating = true;
	OutResult.Time = 0.f;
	OutResult.Distance = 0.f;

	// Return start location as 'safe location'
	OutResult.Location = QueryTM.GetLocation();

	const bool bValidPosition = !!(GetFlags(Hit) & EHitFlags::Position);
	if (bValidPosition)
	{
		const FVector HitPosition = GetPosition(Hit);
		const bool bFinitePosition = !HitPosition.ContainsNaN();
		if (bFinitePosition)
		{
			OutResult.ImpactPoint = HitPosition;
		}
		else
		{
			OutResult.ImpactPoint = StartLoc;
			UE_LOGF(LogPhysics, Verbose, "Warning: ConvertOverlappedShapeToImpactHit: MTD returned NaN :( position: %ls", *HitPosition.ToString());
		}
	}
	else
	{
		OutResult.ImpactPoint = StartLoc;
	}
	OutResult.TraceStart = StartLoc;
	OutResult.TraceEnd = EndLoc;

	const FVector HitNormal = GetNormal(Hit);
	const bool bFiniteNormal = !HitNormal.ContainsNaN();
	const bool bHasNormal = (GetFlags(Hit) & EHitFlags::Normal) != EHitFlags::None;
	const bool bValidNormal = bHasNormal && bFiniteNormal;

	// Use MTD result if possible. We interpret the MTD vector as both the direction to move and the opposing normal.
	if (bValidNormal)
	{
		OutResult.ImpactNormal = HitNormal;
		OutResult.PenetrationDepth = FMath::Abs(GetDistance(Hit));
	}
	else
	{
		// Fallback normal if we can't find it with MTD or otherwise.
		OutResult.ImpactNormal = FVector::UpVector;
		OutResult.PenetrationDepth = 0.f;
		if (!bFiniteNormal)
		{
			UE_LOGF(LogPhysics, Verbose, "Warning: ConvertOverlappedShapeToImpactHit: MTD returned NaN :( normal: %ls", *HitNormal.ToString());
		}
	}

#if DRAW_OVERLAPPING_TRIS
	if (CVarShowInitialOverlaps != 0 && World && World->IsGameWorld())
	{
		DrawOverlappingTris(World, Hit, Geom, QueryTM);
	}
#endif

	if (bBlockingHit)
	{
		// Zero-distance hits are often valid hits and we can extract the hit normal.
		// For invalid normals we can try other methods as well (get overlapping triangles).
		if (GetDistance(Hit) == 0.f || !bValidNormal)	//todo(ocohen): isn't hit distance always zero in this function? should this be if(!bValidNormal) ?
		{
			ComputeZeroDistanceImpactNormalAndPenetration(World, Hit, Geom, QueryTM, OutResult);
		}
	}
	else
	{
		// non blocking hit (overlap).
		if (!bValidNormal)
		{
			OutResult.ImpactNormal = (StartLoc - EndLoc).GetSafeNormal();
			// Since zero-length sweeps are allowed, it is possible that we could compute a zero vector for the normal.
			// We want to always return a valid normal, however we only want to assert if a normal is requested but not able to be computed.
			const bool bIsNormalized = OutResult.ImpactNormal.IsNormalized();
			if (!bIsNormalized)
			{
				if (bHasNormal)
				{
					ensureMsgf(false, TEXT("Unable to compute valid normal from start/end when normal was requested. Start: %s, End: %s."), 
						*StartLoc.ToString(), *EndLoc.ToString());
				}
				OutResult.ImpactNormal = FVector::UpVector;
			}
		}
	}

	OutResult.Normal = OutResult.ImpactNormal;
	
	const FPhysicsActor* GTHitActor = GetGTActor(&HitActor);

	//Note that a non-gt query could still happen on gt if we're in a fixed tick
	constexpr bool bIsGTQuery = std::is_same<THitLocation, FHitLocation>::value;
	const FPhysicsShape* GTHitShape = GetGTShape<bIsGTQuery>(&HitShape, GTHitActor);
	if(GTHitActor && GTHitShape)
	{
		SetHitResultFromShapeAndFaceIndex(*GTHitShape, *GTHitActor, GetInternalFaceIndex(Hit), OutResult.ImpactPoint, OutResult, bReturnPhysMat);
	}

	return bBlockingHit;
}


void ConvertQueryOverlap(const FPhysicsShape& Shape, const FPhysicsActor& Actor, FOverlapResult& OutOverlap, const Chaos::Filter::FQueryFilterData& QueryFilter)
{
	using namespace ChaosInterface;

	// Grab actor/component
	OutOverlap.PhysicsObject = nullptr;
	
	if (!HasValidUserData(Actor))
	{
		return;
	}
	// Try body instance
	else if (const FBodyInstance* BodyInst = GetUserData(Actor))
	{
		BodyInst = FPhysicsInterface::ShapeToOriginalBodyInstance(BodyInst, &Shape);
		if (BodyInst->OwnerComponent.IsValid())
		{
			// Create a handle that needs to be resolved later when accessed in game thread,
			// since resolving FActorInstanceHandle may require to access related UObjects (e.g. owner actor), which is not safe outside game thread.
			OutOverlap.OverlapObjectHandle = FActorInstanceHandle::MakeActorHandleToResolve(BodyInst->OwnerComponent, BodyInst->InstanceBodyIndex);
			OutOverlap.Component = BodyInst->OwnerComponent; // Copying weak pointer is faster than assigning raw pointer.
		}

		// Use GetActorReferenceWithWelding() instead of GetPhysicsActor() so that
		// welded children resolve to their parent's PhysicsObject rather than returning null.
		// When a body is welded, its shapes are merged into the parent particle,
		// so the parent's PhysicsObject is the correct one for force/impulse application.
		const FPhysicsActorHandle& PhysicsActor = BodyInst->GetActorReferenceWithWelding();
		OutOverlap.PhysicsObject = PhysicsActor ? PhysicsActor->GetPhysicsObject() : nullptr;

		if (OutOverlap.PhysicsObject != nullptr)
		{
			OutOverlap.PhysicsObjectOwner = OutOverlap.Component;
			if (FChaosUserEntityAppend* ChaosUserEntityAppend = FChaosUserData::Get<FChaosUserEntityAppend>(Actor.UserData()))
			{
				OutOverlap.PhysicsObjectOwner = ChaosUserEntityAppend->GetOwnerObject();
			}
		}

		// Set ItemIndex
		int32 ItemIndex = INDEX_NONE;
		// To avoid migrations, we always set the body index and we defer bMultiBodyOverlap to the getter
#if UE_WITH_REMOTE_OBJECT_HANDLE
		ItemIndex  = BodyInst->InstanceBodyIndex;
#else
		if (const IPhysicsBodyInstanceOwner* BodyInstanceOwner = IPhysicsBodyInstanceOwner::GetPhysicsBodyInstanceOwnerFromOverlapResult(OutOverlap))
		{
			const bool bUseBodyIndex = BodyInstanceOwner->IsMultiBodyOverlap();
			ItemIndex = bUseBodyIndex ? BodyInst->InstanceBodyIndex : INDEX_NONE;
		}
#endif
		OutOverlap.SetItemIndex(ItemIndex);
	}
	else
	{
		// Currently geom collections are registered with a primitive component user data, but maybe custom should be adapted
		// to be more general so we can support leaf identification #BGTODO
		UPrimitiveComponent* PossibleOwner = GetPrimitiveComponentFromUserData(Actor);

		if(PossibleOwner)
		{
			// Create a handle that needs to be resolved later when accessed in game thread,
			// since resolving FActorInstanceHandle may require to access related UObjects (e.g. owner actor), which is not safe outside game thread.
			OutOverlap.OverlapObjectHandle = FActorInstanceHandle::MakeActorHandleToResolve(PossibleOwner, INDEX_NONE);
			OutOverlap.Component = PossibleOwner;
			OutOverlap.SetItemIndex(INDEX_NONE);

			// If this is a geometry collection, we can do more to extract the properly ItemIndex.
			if (const IPhysicsProxyBase* ActorProxy = Actor.GetProxy())
			{
				if (ActorProxy->GetType() == EPhysicsProxyType::GeometryCollectionType)
				{
					const FGeometryCollectionPhysicsProxy* ConcreteProxy = static_cast<const FGeometryCollectionPhysicsProxy*>(ActorProxy);
					const FGeometryCollectionItemIndex ItemIndex = ConcreteProxy->GetItemIndexFromGTParticle_External(Actor.CastToRigidParticle());
					OutOverlap.SetItemIndex(ItemIndex.GetItemIndex());
				}
			}

		}
		else
		{
			ensureMsgf(false, TEXT("ConvertQueryOverlap called with bad payload type"));
		}
	}
}

/** Util to add NewOverlap to OutOverlaps if it is not already there */
static void AddUniqueOverlap(TArray<FOverlapResult>& OutOverlaps, const FOverlapResult& NewOverlap)
{
	// Look to see if we already have this overlap (based on component)
	for(int32 TestIdx=0; TestIdx<OutOverlaps.Num(); TestIdx++)
	{
		FOverlapResult& Overlap = OutOverlaps[TestIdx];

		if (FOverlapKey(Overlap) == FOverlapKey(NewOverlap))
		{
			// These should be the same if the component matches!
			checkSlow(Overlap.OverlapObjectHandle == NewOverlap.OverlapObjectHandle);

			// If we had a non-blocking overlap with this component, but now we have a blocking one, use that one instead!
			if(!Overlap.bBlockingHit && NewOverlap.bBlockingHit)
			{
				Overlap = NewOverlap;
			}

			return;
		}
	}

	// Not found, so add it 
	OutOverlaps.Add(NewOverlap);
}

bool IsBlocking(const FPhysicsShape& Shape, const Chaos::Filter::FQueryFilterData& QueryFilter)
{
	using namespace ChaosInterface;

	// See if this is a 'blocking' hit
	const Chaos::Filter::FShapeFilterData ShapeFilter = GetShapeFilterData(Shape);
	const ECollisionQueryHitType HitType = FCollisionQueryFilterCallback::CalcQueryHitType(QueryFilter, ShapeFilter);
	const bool bBlock = (HitType == ECollisionQueryHitType::Block);
	return bBlock;
}

/** Min number of overlaps required to start using a TMap for deduplication */
int32 GNumOverlapsRequiredForTMap = 3;

static FAutoConsoleVariableRef GTestOverlapSpeed(
	TEXT("Engine.MinNumOverlapsToUseTMap"),
	GNumOverlapsRequiredForTMap,
	TEXT("Min number of overlaps required before using a TMap for deduplication")
	);

template <typename THitOverlap>
bool ConvertOverlapResultsImp(int32 NumOverlaps, THitOverlap* OverlapResults, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FOverlapResult>& OutOverlaps)
{
	SCOPE_CYCLE_COUNTER(STAT_CollisionConvertOverlap);

	const int32 ExpectedSize = OutOverlaps.Num() + NumOverlaps;
	OutOverlaps.Reserve(ExpectedSize);
	bool bBlockingFound = false;
	constexpr bool bIsGTQuery = std::is_same<THitOverlap, FHitOverlap>::value;

	if (ExpectedSize >= GNumOverlapsRequiredForTMap)
	{
		// Map from an overlap to the position in the result array (the index has one added to it so 0 can be a sentinel)
		TMap<FOverlapKey, int32, TInlineSetAllocator<64>> OverlapMap;
		OverlapMap.Reserve(ExpectedSize);

		// Fill in the map with existing hits
		for (int32 ExistingIndex = 0; ExistingIndex < OutOverlaps.Num(); ++ExistingIndex)
		{
			const FOverlapResult& ExistingOverlap = OutOverlaps[ExistingIndex];
			OverlapMap.Add(FOverlapKey(ExistingOverlap), ExistingIndex + 1);
		}

		for (int32 PResultIndex = 0; PResultIndex < NumOverlaps; ++PResultIndex)
		{
			FOverlapResult NewOverlap;
			const auto Actor = GetActor(OverlapResults[PResultIndex]);
			const FPhysicsShape* Shape = GetShape(OverlapResults[PResultIndex]);

			const bool bBlock = IsBlocking(*Shape, QueryFilter);
			NewOverlap.bBlockingHit = bBlock;
			bBlockingFound |= bBlock;

			const FPhysicsActor* GTActor = GetGTActor(Actor);
			const FPhysicsShape* GTShape = GetGTShape<bIsGTQuery>(Shape, GTActor);
			if(GTActor && GTShape)
			{
				ConvertQueryOverlap(*GTShape, *GTActor, NewOverlap, QueryFilter);
			}

			// Look for it in the map, newly added elements will start with 0, so we know we need to add it to the results array then (the index is stored as +1)
			//TODO: this doesn't de-duplicate if FOverlapResult has no component - like in PT for example. If we care about de-duplication we should use the PT handle if needed
			//Question: why do we de-duplicate? This seems expensive and is not consistent with traces. The user could handle this at their level anyway (at the cost of a bit of transient memory here)
			//TODO: make sure it doesn't de-duplicate null results
			int32& DestinationIndex = OverlapMap.FindOrAdd(FOverlapKey(NewOverlap));
			if (DestinationIndex == 0)
			{
				DestinationIndex = OutOverlaps.Add(NewOverlap) + 1;
			}
			else
			{
				FOverlapResult& ExistingOverlap = OutOverlaps[DestinationIndex - 1];

				// If we had a non-blocking overlap with this component, but now we have a blocking one, use that one instead!
				if (!ExistingOverlap.bBlockingHit && NewOverlap.bBlockingHit)
				{
					ExistingOverlap = NewOverlap;
				}
			}
		}
	}
	else
	{
		// N^2 approach, no maps
		for (int32 i = 0; i < NumOverlaps; i++)
		{
			FOverlapResult NewOverlap;
			const auto Actor = GetActor(OverlapResults[i]);
			const FPhysicsShape* Shape = GetShape(OverlapResults[i]);

			const bool bBlock = IsBlocking(*Shape, QueryFilter);
			NewOverlap.bBlockingHit = bBlock;
			bBlockingFound |= bBlock;

			const FPhysicsActor* GTActor = GetGTActor(Actor);
			const FPhysicsShape* GTShape = GetGTShape<bIsGTQuery>(Shape, GTActor);
			if (GTActor && GTShape)
			{
				ConvertQueryOverlap(*GTShape, *GTActor, NewOverlap, QueryFilter);
			}

			AddUniqueOverlap(OutOverlaps, NewOverlap);
		}
	}

	return bBlockingFound;
}

bool ConvertOverlapResults(int32 NumOverlaps, FHitOverlap* OverlapResults, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FOverlapResult>& OutOverlaps)
{
	return ConvertOverlapResultsImp(NumOverlaps, OverlapResults, QueryFilter, OutOverlaps);
}

bool ConvertOverlapResults(int32 NumOverlaps, ChaosInterface::FPTOverlapHit* OverlapResults, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FOverlapResult>& OutOverlaps)
{
	return ConvertOverlapResultsImp(NumOverlaps, OverlapResults, QueryFilter, OutOverlaps);
}

FHitResult ConvertOverlapToHitResult(const FOverlapResult& Overlap)
{
	FHitResult Hit;
	Hit.bBlockingHit = Overlap.bBlockingHit;
	Hit.Item = GetItemIndex(Overlap);
	Hit.Component = Overlap.Component;
	Hit.PhysicsObject = Overlap.PhysicsObject;
	Hit.PhysicsObjectOwner = Overlap.PhysicsObjectOwner;
	Hit.HitObjectHandle = Overlap.OverlapObjectHandle;
	return Hit;
}

bool FCompareFHitResultTime::operator()(const FHitResult& A, const FHitResult& B) const
{
	if (A.Time == B.Time)
	{
		// Sort blocking hits after non-blocking hits, if they are at the same time. Also avoid swaps if they are the same.
		// This is important so initial touches are reported before processing stops on the first blocking hit.
		return (A.bBlockingHit == B.bBlockingHit) ? true : B.bBlockingHit;
	}

	return A.Time < B.Time;
}
