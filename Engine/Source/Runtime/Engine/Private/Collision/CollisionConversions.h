// Copyright Epic Games, Inc. All Rights Reserved.
// Utilities to convert from Chaos result structs to Unreal ones

#pragma once 

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "EngineDefines.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsPublic.h"

enum class EConvertQueryResult
{
	Valid,
	Invalid
};

/** 
 * Util to convert single chaos hit (raycast or sweep) to unreal hit result
 *
 * @param	PHit		Chaos hit data
 * @param	OutResult	(out) Result converted
 * @param	CheckLength	Distance of trace
 * @param	QueryFilter	Query Filter 
 * @param	StartLoc	Start of trace
 * @param	EndLoc
 * @param	Geom
 * @param	bReturnFaceIndex	True if we want to lookup the face index
 * @param	bReturnPhysMat		True if we want to lookup the physical material
 * @return	Whether result passed NaN/Inf checks.
 */
EConvertQueryResult ConvertQueryImpactHit(const UWorld* World, const FHitLocation& PHit, FHitResult& OutResult, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, bool bReturnFaceIndex, bool bReturnPhysMat);
EConvertQueryResult ConvertQueryImpactHit(const UWorld* World, const ChaosInterface::FPTLocationHit& PHit, FHitResult& OutResult, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, bool bReturnFaceIndex, bool bReturnPhysMat);

/** 
 * Util to convert Chaos sweep results to unreal hit results and add to array
 *
 * @param	OutHasValidBlockingHit set to whether there is a valid blocking hit found in the results.
 * @param	NumHits		Number of Hits
 * @param	Hits		Array of hits
 * @param	CheckLength	Distance of trace
 * @param	QueryFilter	Query Filter 
 * @param	StartLoc	Start of trace
 * @param	EndLoc		End of trace
 * @param	Geom
 * @param	bReturnFaceIndex	True if we want to lookup the face index
 * @param	bReturnPhysMat		True if we want to lookup the physical material
 * @return	Whether all results passed NaN/Inf checks.
 */
template <typename Hit>
EConvertQueryResult ConvertTraceResults(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, Hit* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FHitResult>& OutHits, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);

template <typename Hit>
EConvertQueryResult ConvertTraceResults(bool& OutHasValidBlockingHit, const UWorld* World, int32 NumHits, Hit* Hits, float CheckLength, const Chaos::Filter::FQueryFilterData& QueryFilter, FHitResult& OutHits, const FVector& StartLoc, const FVector& EndLoc, const FPhysicsGeometry* Geom, const FTransform& QueryTM, float MaxDistance, bool bReturnFaceIndex, bool bReturnPhysMat);

/** 
 * Util to convert Chaos overlap query to our overlap result
 *
 * @param	PShape		Shape that overlaps
 * @param	PActor		Specific actor as PShape might be shared among many actors
 * @param	OutOverlap	(out) Result converted
 * @param	QueryFilter	Query Filter 
 */
void ConvertQueryOverlap(const UWorld* World, const FPhysicsShape& PShape, const FPhysicsActorHandle& PActor, FOverlapResult& OutOverlap, const Chaos::Filter::FQueryFilterData& QueryFilter);

/**
 * Util to determine if a shape is deemed blocking based on the query filter
 *
 * @param PShape Shape that overlaps
 * @QueryFilter Query Filter
 * @return true if the query filter and shape filter resolve to be blocking
 */
bool IsBlocking(const FPhysicsShape& PShape, const Chaos::Filter::FQueryFilterData& QueryFilter);


/** 
 * Util to convert a list of overlap hits into FOverlapResult and add them to OutOverlaps, if not already there
 *
 * @param	NumOverlaps	Number Of Overlaps that happened
 * @param	POverlapResults			Overlaps list
 * @param	QueryFilter	Query filter for converting
 * @return	OutOverlaps	Converted data
 */
bool ConvertOverlapResults(int32 NumOverlaps, FHitOverlap* POverlapResults, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FOverlapResult>& OutOverlaps);
bool ConvertOverlapResults(int32 NumOverlaps, ChaosInterface::FPTOverlapHit* POverlapResults, const Chaos::Filter::FQueryFilterData& QueryFilter, TArray<FOverlapResult>& OutOverlaps);

/**
 * Converts an overlap result to a hit result.
 */
FHitResult ConvertOverlapToHitResult(const FOverlapResult& Overlap);

struct FCompareFHitResultTime
{
	bool operator()(const FHitResult& A, const FHitResult& B) const;
};

#ifndef DRAW_OVERLAPPING_TRIS
#define DRAW_OVERLAPPING_TRIS (!(UE_BUILD_SHIPPING || UE_BUILD_TEST))
#endif