// Copyright Epic Games, Inc. All Rights Reserved.
// Utilities to convert from PhysX result structs to Unreal ones

#pragma once 

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "EngineDefines.h"
