// Copyright Epic Games, Inc. All Rights Reserved.

#include "Collision/CollisionDebugDrawing.h"
#include "Components/PrimitiveComponent.h"
#include "DrawDebugHelpers.h"
#include "Engine/OverlapResult.h"

#include "Chaos/Capsule.h"
#include "WorldCollision.h"


#if ENABLE_DRAW_DEBUG

namespace CollisionDebugDrawing
{
	static FColor TraceColor(255,255,255);
	static FColor HitColor(156, 179, 209);
	static FColor PenetratingColor(64, 64, 255);
	static FColor BlockColor(255,64,64);
	static FColor TouchColor(64,255,64);
	static float NormalLength = 20.f;
}

ENGINE_API void DrawLineTraces(const UWorld* InWorld, const FVector& Start, const FVector& End, const TArray<FHitResult>& Hits, float Lifetime)
{
	FColor Color = (Hits.Num() > 0)? CollisionDebugDrawing::HitColor: CollisionDebugDrawing::TraceColor;
	DrawDebugLine(InWorld, Start, End, Color, false, Lifetime);

	for(int32 HitIdx=0; HitIdx<Hits.Num(); HitIdx++)
	{
		const FHitResult& Hit = Hits[HitIdx];

		FVector NormalStart = Hit.Location;
		FVector NormalEnd = NormalStart + (Hit.Normal * CollisionDebugDrawing::NormalLength);
		const FColor NormalColor = Hit.bBlockingHit ? CollisionDebugDrawing::BlockColor : CollisionDebugDrawing::TouchColor;
		DrawDebugDirectionalArrow(InWorld, NormalStart, NormalEnd, 5.f, NormalColor, false, Lifetime);
		NormalStart = Hit.ImpactPoint;
		NormalEnd = NormalStart + (Hit.ImpactNormal * CollisionDebugDrawing::NormalLength);
		DrawDebugDirectionalArrow(InWorld, NormalStart, NormalEnd, 5.f, NormalColor, false, Lifetime);

		UE_LOGF(LogCollision, Log, "  %d: T=%f C='%ls' BLOCK=%d", HitIdx, Hit.Time, *GetPathNameSafe(Hit.Component.Get()), Hit.bBlockingHit);
	}
}

ENGINE_API void DrawSphereSweeps(const UWorld* InWorld, const FVector& Start, const FVector& End, const float Radius, const TArray<FHitResult>& Hits, float Lifetime)
{
	FColor Color = (Hits.Num() > 0)? CollisionDebugDrawing::HitColor: CollisionDebugDrawing::TraceColor;
	DrawDebugSphere(InWorld, Start, Radius, FMath::Max(Radius/4.f, 2.f), Color, false, Lifetime);
	DrawDebugSphere(InWorld, End, Radius, FMath::Max(Radius/4.f, 2.f), Color, false, Lifetime);
	DrawDebugLine(InWorld, Start+FVector(0, 0, Radius), End+FVector(0, 0, Radius), Color, false, Lifetime);
	DrawDebugLine(InWorld, Start-FVector(0, 0, Radius), End-FVector(0, 0, Radius), Color, false, Lifetime);
	for(int32 HitIdx=0; HitIdx<Hits.Num(); HitIdx++)
	{
		const FHitResult& Hit = Hits[HitIdx];

		FVector NormalStart = Hit.Location;
		FVector NormalEnd = NormalStart + (Hit.Normal * CollisionDebugDrawing::NormalLength);
		const FColor NormalColor = Hit.bBlockingHit ? CollisionDebugDrawing::BlockColor : CollisionDebugDrawing::TouchColor;
		DrawDebugDirectionalArrow(InWorld, NormalStart, NormalEnd, 5.f, NormalColor, false, Lifetime);
		NormalStart = Hit.ImpactPoint;
		NormalEnd = NormalStart + (Hit.ImpactNormal * CollisionDebugDrawing::NormalLength);
		DrawDebugDirectionalArrow(InWorld, NormalStart, NormalEnd, 5.f, NormalColor, false, Lifetime);

		//UE_LOGF(LogCollision, Log, "  %d: T=%f C='%ls' BLOCK=%d", HitIdx, Hit.Time, Hit.Component ? *Hit.Component->GetPathName() : TEXT("NOCOMP"), Hit.bBlockingHit);
	}
}

ENGINE_API void DrawBoxSweeps(const UWorld* InWorld, const FVector& Start, const FVector& End, const FVector& Extent, const FQuat& Rot, const TArray<FHitResult>& Hits, float Lifetime)
{
	FBox StartBox(Start-Extent, Start+Extent);
	FBox EndBox(End-Extent, End+Extent);

	FColor Color = (Hits.Num() > 0)? CollisionDebugDrawing::HitColor: CollisionDebugDrawing::TraceColor;

	DrawDebugLine(InWorld, Start, End, Color, false, Lifetime);
	DrawDebugBox(InWorld, StartBox.GetCenter(), StartBox.GetExtent(), Rot, Color, false, Lifetime);
	DrawDebugBox(InWorld, EndBox.GetCenter(), EndBox.GetExtent(), Rot, Color, false, Lifetime);
	DrawDebugLine(InWorld, StartBox.Min, EndBox.Min, Color, false, Lifetime);
	DrawDebugLine(InWorld, FVector(StartBox.Min.X, StartBox.Min.Y, StartBox.Max.Z), FVector(EndBox.Min.X, EndBox.Min.Y, EndBox.Max.Z), Color, false, Lifetime);
	DrawDebugLine(InWorld, FVector(StartBox.Min.X, StartBox.Max.Y, StartBox.Max.Z), FVector(EndBox.Min.X, EndBox.Max.Y, EndBox.Max.Z), Color, false, Lifetime);
	DrawDebugLine(InWorld, FVector(StartBox.Min.X, StartBox.Max.Y, StartBox.Min.Z), FVector(EndBox.Min.X, EndBox.Max.Y, EndBox.Min.Z), Color, false, Lifetime);
	DrawDebugLine(InWorld, FVector(StartBox.Max.X, StartBox.Max.Y, StartBox.Min.Z), FVector(EndBox.Max.X, EndBox.Max.Y, EndBox.Min.Z), Color, false, Lifetime);
	DrawDebugLine(InWorld, FVector(StartBox.Max.X, StartBox.Min.Y, StartBox.Min.Z), FVector(EndBox.Max.X, EndBox.Min.Y, EndBox.Min.Z), Color, false, Lifetime);
	DrawDebugLine(InWorld, FVector(StartBox.Max.X, StartBox.Min.Y, StartBox.Max.Z), FVector(EndBox.Max.X, EndBox.Min.Y, EndBox.Max.Z), Color, false, Lifetime);
	DrawDebugLine(InWorld, StartBox.Max, EndBox.Max, Color, false, Lifetime);

	for(int32 HitIdx=0; HitIdx<Hits.Num(); HitIdx++)
	{
		const FHitResult& Hit = Hits[HitIdx];

		FVector NormalStart = Hit.Location;
		FVector NormalEnd = NormalStart + (Hit.Normal * CollisionDebugDrawing::NormalLength);
		const FColor NormalColor = Hit.bBlockingHit ? CollisionDebugDrawing::BlockColor : CollisionDebugDrawing::TouchColor;
		DrawDebugDirectionalArrow(InWorld, NormalStart, NormalEnd, 5.f, NormalColor, false, Lifetime);
		NormalStart = Hit.ImpactPoint;
		NormalEnd = NormalStart + (Hit.ImpactNormal * CollisionDebugDrawing::NormalLength);
		DrawDebugDirectionalArrow(InWorld, NormalStart, NormalEnd, 5.f, NormalColor, false, Lifetime);

		//UE_LOGF(LogCollision, Log, "  %d: T=%f C='%ls' BLOCK=%d", HitIdx, Hit.Time, Hit.Component ? *Hit.Component->GetPathName() : TEXT("NOCOMP"), Hit.bBlockingHit);
	}
}

ENGINE_API void DrawCapsuleSweeps(const UWorld* InWorld, const FVector& Start, const FVector& End, float HalfHeight, float Radius, const FQuat& Rotation, const TArray<FHitResult>& Hits, float Lifetime)
{
	FColor Color = (Hits.Num() > 0)? CollisionDebugDrawing::HitColor: CollisionDebugDrawing::TraceColor;
	if (Hits.Num() > 0 && Hits[0].bBlockingHit)
	{
		Color = CollisionDebugDrawing::PenetratingColor;
		//		LifeTime=0.f;
	}
	DrawDebugLine(InWorld, Start, End, Color, false, Lifetime);
	DrawDebugCapsule(InWorld, Start, HalfHeight, Radius, Rotation, Color, false, Lifetime);
	DrawDebugCapsule(InWorld, End, HalfHeight, Radius, Rotation, Color, false, Lifetime);

	FVector CenterToEndTip = Rotation.RotateVector(FVector(0, 0, HalfHeight));

	DrawDebugLine(InWorld, Start + CenterToEndTip, End + CenterToEndTip, Color, false, Lifetime);
	DrawDebugLine(InWorld, Start - CenterToEndTip, End - CenterToEndTip, Color, false, Lifetime);

	FVector Dir = (End-Start);
	Dir.Normalize();
	FVector Up(0, 0, 1);
	FVector Right = Dir ^ Up; 
	Right *= Radius;

	DrawDebugLine(InWorld, Start - Right, End - Right, CollisionDebugDrawing::TraceColor, false, Lifetime);
	DrawDebugLine(InWorld, Start + Right, End + Right, CollisionDebugDrawing::TraceColor, false, Lifetime);

	for(int32 HitIdx=0; HitIdx<Hits.Num(); HitIdx++)
	{
		const FHitResult& Hit = Hits[HitIdx];

		FVector NormalStart = Hit.Location;
		FVector NormalEnd = NormalStart + (Hit.Normal * CollisionDebugDrawing::NormalLength);
		const FColor NormalColor = Hit.bBlockingHit ? CollisionDebugDrawing::BlockColor : CollisionDebugDrawing::TouchColor;
		DrawDebugDirectionalArrow(InWorld, NormalStart, NormalEnd, 5.f, NormalColor, false, Lifetime);
		NormalStart = Hit.ImpactPoint;
		NormalEnd = NormalStart + (Hit.ImpactNormal * CollisionDebugDrawing::NormalLength);
		DrawDebugDirectionalArrow(InWorld, NormalStart, NormalEnd, 5.f, FColor(255, 255, 0), false, Lifetime);

		//UE_LOGF(LogCollision, Log, "  %d: T=%f C='%ls' BLOCK=%d", HitIdx, Hit.Time, Hit.Component ? *Hit.Component->GetPathName() : TEXT("NOCOMP"), Hit.bBlockingHit);
	}
}

ENGINE_API void DrawBoxOverlap(const UWorld* InWorld, const FVector& Pos, const FVector& Extent, const FQuat& Rot, TArray<struct FOverlapResult>& Overlaps, float Lifetime)
{
	FColor Color = (Overlaps.Num() > 0)? CollisionDebugDrawing::HitColor: CollisionDebugDrawing::TraceColor;
	DrawDebugBox(InWorld, Pos, Extent, Rot, Color, false, Lifetime);

	for(int32 OverlapIdx=0; OverlapIdx<Overlaps.Num(); OverlapIdx++)
	{
		const FOverlapResult& Overlap = Overlaps[OverlapIdx];

		if (Overlap.Component.Get())
		{
			const FColor OverlapColor = Overlap.bBlockingHit ? CollisionDebugDrawing::BlockColor : CollisionDebugDrawing::TouchColor;
			DrawDebugDirectionalArrow(InWorld, Pos, Overlap.Component->GetComponentLocation(), 5.f, OverlapColor, false, Lifetime);
		}

		//UE_LOGF(LogCollision, Log, "  %d: C='%ls' BLOCK=%d", OverlapIdx, Overlap.Component ? *Overlap.Component->GetPathName() : TEXT("NOCOMP"), Overlap.bBlockingHit);
	}
}

ENGINE_API void DrawSphereOverlap(const UWorld* InWorld, const FVector& Pos, const float Radius, TArray<struct FOverlapResult>& Overlaps, float Lifetime)
{
	FColor Color = (Overlaps.Num() > 0)? CollisionDebugDrawing::HitColor: CollisionDebugDrawing::TraceColor;
	DrawDebugSphere(InWorld, Pos, Radius, FMath::Max(Radius/4.f, 2.f), Color, false, Lifetime);

	for(int32 OverlapIdx=0; OverlapIdx<Overlaps.Num(); OverlapIdx++)
	{
		const FOverlapResult& Overlap = Overlaps[OverlapIdx];

		if (Overlap.Component.Get())
		{
			const FColor OverlapColor = Overlap.bBlockingHit ? CollisionDebugDrawing::BlockColor : CollisionDebugDrawing::TouchColor;
			DrawDebugDirectionalArrow(InWorld, Pos, Overlap.Component->GetComponentLocation(), 5.f, OverlapColor, false, Lifetime);
		}

		//UE_LOGF(LogCollision, Log, "  %d: C='%ls' BLOCK=%d", OverlapIdx, Overlap.Component ? *Overlap.Component->GetPathName() : TEXT("NOCOMP"), Overlap.bBlockingHit);
	}
}

ENGINE_API void DrawCapsuleOverlap(const UWorld* InWorld,const FVector& Pos, const float HalfHeight, const float Radius, const FQuat& Rot, TArray<struct FOverlapResult>& Overlaps, float Lifetime)
{
	FColor Color = (Overlaps.Num() > 0)? CollisionDebugDrawing::HitColor: CollisionDebugDrawing::TraceColor;
	DrawDebugCapsule(InWorld, Pos, HalfHeight, Radius, Rot, Color, false, Lifetime);

	for(int32 OverlapIdx=0; OverlapIdx<Overlaps.Num(); OverlapIdx++)
	{
		const FOverlapResult& Overlap = Overlaps[OverlapIdx];

		if (Overlap.Component.Get())
		{
			const FColor OverlapColor = Overlap.bBlockingHit ? CollisionDebugDrawing::BlockColor : CollisionDebugDrawing::TouchColor;
			DrawDebugDirectionalArrow(InWorld, Pos, Overlap.Component->GetComponentLocation(), 5.f, OverlapColor, false, Lifetime);
		}

		//UE_LOGF(LogCollision, Log, "  %d: C='%ls' BLOCK=%d", OverlapIdx, Overlap.Component ? *Overlap.Component->GetPathName() : TEXT("NOCOMP"), Overlap.bBlockingHit);
	}
}

void DrawGeomOverlaps(const UWorld* InWorld, const Chaos::FImplicitObject& Geom, const FTransform& GeomPose, TArray<struct FOverlapResult>& Overlaps, float Lifetime)
{
	using namespace Chaos;
	FVector Pos = GeomPose.GetLocation();
	FQuat Rot = GeomPose.GetRotation();

	if (const auto Box = Geom.template GetObject<TBox<FReal,3>>())
	{
		DrawBoxOverlap(InWorld, Pos, Box->Extents() * 0.5f, Rot, Overlaps, Lifetime);
	}
	else if (const auto Sphere = Geom.template GetObject<Chaos::FSphere>())
	{
		DrawSphereOverlap(InWorld, Pos, Sphere->GetRadiusf(), Overlaps, Lifetime);
	}
	else if (const auto Capsule = Geom.template GetObject<FCapsule>())
	{
		// Convert here from Chaos to unreal definition of capsule height
		DrawCapsuleOverlap(InWorld, Pos, Capsule->GetHeightf() * 0.5f + Capsule->GetRadiusf(), Capsule->GetRadiusf(), Rot, Overlaps, Lifetime);
	}
}

void DrawGeomSweeps(const UWorld* InWorld, const FVector& Start, const FVector& End, const Chaos::FImplicitObject& Geom, const FQuat& Rotation, const TArray<FHitResult>& Hits, float Lifetime)
{
	using namespace Chaos;
	if (const auto Box = Geom.template GetObject<TBox<FReal, 3>>())
	{
		DrawBoxSweeps(InWorld, Start, End, Box->Extents() * 0.5f, Rotation, Hits, Lifetime);
	}
	else if (const Chaos::FSphere* Sphere = Geom.template GetObject<Chaos::FSphere>())
	{
		DrawSphereSweeps(InWorld, Start, End, Sphere->GetRadiusf(), Hits, Lifetime);
	}
	else if (const auto Capsule = Geom.template GetObject<FCapsule>())
	{
		// Convert here from Chaos to unreal definition of capsule height
		DrawCapsuleSweeps(InWorld, Start, End, Capsule->GetHeightf() * 0.5f + Capsule->GetRadiusf(), Capsule->GetRadiusf(), Rotation, Hits, Lifetime);
	}
}

#endif // ENABLE_DRAW_DEBUG
