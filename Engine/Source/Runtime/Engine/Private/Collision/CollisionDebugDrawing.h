// Copyright Epic Games, Inc. All Rights Reserved.
// Draw functions for debugging trace/sweeps/overlaps


#pragma once 

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "PhysicsPublic.h"

struct FHitResult;

namespace Chaos
{
	class FImplicitObject;
}

void DrawGeomOverlaps(const UWorld* InWorld, const Chaos::FImplicitObject& Geom, const FTransform& GeomPose, TArray<struct FOverlapResult>& Overlaps, float Lifetime);
void DrawGeomSweeps(const UWorld* InWorld, const FVector& Start, const FVector& End, const Chaos::FImplicitObject& Geom, const FQuat& Rotation, const TArray<FHitResult>& Hits, float Lifetime);
