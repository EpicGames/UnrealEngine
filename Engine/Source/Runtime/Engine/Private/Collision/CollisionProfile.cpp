// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/CollisionProfile.h"
#include "Misc/ConfigCacheIni.h"
#include "Components/PrimitiveComponent.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollisionProfile)

DEFINE_LOG_CATEGORY_STATIC(LogCollisionProfile, Warning, All)

constexpr uint32 MaxNumCollisionChannels = 64;
#define IS_VALID_COLLISIONCHANNEL(x) ((x)> ECC_Destructible && (x) < ECC_OverlapAll_Deprecated)

// do not chnage this name. This value is serialize to other object, if you change, it will mess up serialization, you'll need to fix up name by versioning
FName UCollisionProfile::CustomCollisionProfileName = FName("Custom");

//////////////////////////////////////////////////////////////////////////
FCollisionResponseTemplate::FCollisionResponseTemplate()
	: CollisionEnabled(ECollisionEnabled::NoCollision)
	, ObjectType(ECollisionChannel::ECC_WorldStatic)
	, bCanModify(true)
	, ResponseToChannels()
#if WITH_EDITORONLY_DATA
	, HelpMessage(TEXT("Needs description"))
#endif
{
}

bool FCollisionResponseTemplate::IsEqual(const TEnumAsByte<ECollisionEnabled::Type> InCollisionEnabled, 
	const TEnumAsByte<enum ECollisionChannel> InObjectType, 
	const struct FCollisionResponseContainer& InResponseToChannels)
{
	return (CollisionEnabled == InCollisionEnabled && ObjectType == InObjectType && 
		FMemory::Memcmp(&InResponseToChannels, &ResponseToChannels, sizeof(FCollisionResponseContainer))== 0);
}

//////////////////////////////////////////////////////////////////////////

ENGINE_API const FName UCollisionProfile::NoCollision_ProfileName = FName("NoCollision");
ENGINE_API const FName UCollisionProfile::BlockAll_ProfileName = FName("BlockAll");
ENGINE_API const FName UCollisionProfile::OverlapAll_ProfileName = FName("OverlapAll");
ENGINE_API const FName UCollisionProfile::PhysicsActor_ProfileName = FName("PhysicsActor");
ENGINE_API const FName UCollisionProfile::BlockAllDynamic_ProfileName = FName("BlockAllDynamic");
ENGINE_API const FName UCollisionProfile::Pawn_ProfileName = FName("Pawn");
ENGINE_API const FName UCollisionProfile::Vehicle_ProfileName = FName("Vehicle");
ENGINE_API const FName UCollisionProfile::DefaultProjectile_ProfileName = FName("DefaultProjectile");


UCollisionProfile::UCollisionProfile(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SectionName = TEXT("Collision");
}

UCollisionProfile* UCollisionProfile::Get()
{
	static bool bInitialized = false;
	// this is singletone. Use default object
	UCollisionProfile* CollisionProfile = GetMutableDefault<UCollisionProfile>();

	if (!bInitialized)
	{
		CollisionProfile->LoadProfileConfig();
		bInitialized = true;
	}

	return CollisionProfile;
}

void UCollisionProfile::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		LoadProfileConfig();
	}
}

void UCollisionProfile::GetProfileNames(TArray<TSharedPtr<FName>>& OutNameList)
{
	const UCollisionProfile* CollisionProfile = UCollisionProfile::Get();
	check(CollisionProfile);

	int32 NumProfiles = CollisionProfile->GetNumOfProfiles();

	OutNameList.Empty(NumProfiles);

	for (int32 ProfileId = 0; ProfileId < NumProfiles; ++ProfileId)
	{
		const FCollisionResponseTemplate* ProfileTemplate = CollisionProfile->GetProfileByIndex(ProfileId);
		check(ProfileTemplate);

		OutNameList.Add(MakeShareable(new FName(ProfileTemplate->Name)));
	}
}

bool UCollisionProfile::GetChannelAndResponseParams(FName ProfileName, ECollisionChannel &CollisionChannel, FCollisionResponseParams &ResponseParams)
{
	const UCollisionProfile* CollisionProfile = UCollisionProfile::Get();
	check(CollisionProfile);

	FCollisionResponseTemplate Template;

	if (CollisionProfile->GetProfileTemplate(ProfileName, Template))
	{
		CollisionChannel = Template.ObjectType;
		ResponseParams = FCollisionResponseParams(Template.ResponseToChannels);
		return true;
	}

	// Check for redirects
	const FName* RedirectName = CollisionProfile->LookForProfileRedirect(ProfileName);
	if (RedirectName && CollisionProfile->GetProfileTemplate(*RedirectName, Template))
	{
		CollisionChannel = Template.ObjectType;
		ResponseParams = FCollisionResponseParams(Template.ResponseToChannels);
		return true;
	}

	return false;
}

bool UCollisionProfile::GetProfileTemplate(FName ProfileName, struct FCollisionResponseTemplate& ProfileData) const
{
	// verify if it is in redirect first
	if ( ProfileName != NAME_None )
	{
		return FindProfileData(Profiles, ProfileName, ProfileData);
	}

	return false;
}

bool UCollisionProfile::CheckRedirect(FName ProfileName, FBodyInstance& BodyInstance, struct FCollisionResponseTemplate& Template) const
{
	// make sure we're not setting invalid collision profile name
	if ( FBodyInstance::IsValidCollisionProfileName(ProfileName) )
	{
		const FName* NewName = ProfileRedirectsMap.Find(ProfileName);
		if ( NewName )
		{
			// if it's same as old version
			// we can use redirect
			// otherwise we don't want to use redirect
			BodyInstance.CollisionProfileName = *NewName;

			// if new profile name exists
			if ( *NewName != NAME_None )
			{
				const bool bSuccessfullyFoundProfileData = FindProfileData(Profiles, *NewName, Template);
				check(bSuccessfullyFoundProfileData);
			}

			return true;
		}
	}

	return false;
}

const FName* UCollisionProfile::LookForProfileRedirect(FName ProfileName) const
{
	return ProfileRedirectsMap.Find(ProfileName);
}

bool UCollisionProfile::FindProfileData(const TArray<FCollisionResponseTemplate>& ProfileList, FName ProfileName, struct FCollisionResponseTemplate& ProfileData) const
{
	if ( ProfileName != NAME_None )
	{
		for ( auto Iter = ProfileList.CreateConstIterator(); Iter; ++Iter )
		{
			if ( (*Iter).Name == ProfileName )
			{
				ProfileData = (*Iter);
				return true;
			}
		}
	}

	return false;
}

bool UCollisionProfile::ReadConfig(FName ProfileName, FBodyInstance& BodyInstance) const
{
	FCollisionResponseTemplate Template;

	// first check redirect
	// if that fails, just get profile
	if ( CheckRedirect(ProfileName, BodyInstance, Template) ||
		GetProfileTemplate(ProfileName, Template) )
	{
		// note that this can be called during loading or run-time (because of the function)
		// from property, it just uses property handle to set all data
		// but we can't use functions - i.e. SetCollisionEnabled - 
		// which will reset ProfileName by default
		BodyInstance.CollisionEnabled = Template.CollisionEnabled;
		BodyInstance.ObjectType = Template.ObjectType;
		BodyInstance.CollisionResponses.SetCollisionResponseContainer(Template.ResponseToChannels);
		BodyInstance.UpdatePhysicsFilterData();
		return true;
	}

	return false;
}

const FCollisionResponseTemplate* UCollisionProfile::GetProfileByIndex(int32 Index) const
{ 
	if (Profiles.IsValidIndex(Index))
	{
		return &Profiles[Index]; 
	}

	return NULL;
}

// @todo check with Terrence, if this gets saved or not to local
void UCollisionProfile::AddChannelRedirect(FName OldName, FName NewName)
{
	if(OldName != NewName)
	{
		FName& NewValue = CollisionChannelRedirectsMap.FindOrAdd(OldName);
		NewValue = NewName;

		CollisionChannelRedirects.Empty();

		// now add back to it
		TArray<FString> KeyValues;
		for(auto Iter=CollisionChannelRedirectsMap.CreateConstIterator(); Iter; ++Iter)
		{
			const FName &Key = Iter.Key();
			const FName &Value = Iter.Value();
			CollisionChannelRedirects.Add(FRedirector(Key, Value));
		}

		// if change collision channel redirect, it will requires for all profiles to refresh that data
		for(auto Iter=Profiles.CreateIterator(); Iter; ++Iter)
		{
			SaveCustomResponses(*Iter);
		}
	}
}

void UCollisionProfile::AddProfileRedirect(FName OldName, FName NewName)
{
	if(OldName != NewName)
	{
		ProfileRedirectsMap.FindOrAdd(OldName) = NewName;

		ProfileRedirects.Empty();

		// now add back to it
		TArray<FString> KeyValues;
		for(auto Iter=ProfileRedirectsMap.CreateConstIterator(); Iter; ++Iter)
		{
			const FName &Key = Iter.Key();
			const FName &Value = Iter.Value();
			ProfileRedirects.Add(FRedirector(Key, Value));
		}
	}
}

void UCollisionProfile::LoadProfileConfig(bool bForceInit)
{
	/** 
	 * This function loads all config data to memory
	 * 
	 * 1. First it fixes the meta data for each custom channel name since those meta data is used for #2
	 * 2. Load Default Profile so that it can be used later
	 * 3. Second it sets up Correct ResponseToChannel for all profiles
	 * 4. It loads profile redirect data 
	 **/
	// read "EngineTraceChanne" and "GameTraceChanne" and set meta data
	const FConfigSection* Configs = GConfig->GetSection( TEXT("/Script/Engine.CollisionProfile"), false, GEngineIni );

	OnLoadProfileConfig.Broadcast(this);

	// before any op, verify if profiles contains invalid name - such as Custom profile name - remove all of them
	for (auto Iter=Profiles.CreateConstIterator(); Iter; ++Iter)
	{
		// make sure it doesn't have any 
		if (Iter->Name == CustomCollisionProfileName)
		{
			UE_LOGF(LogCollisionProfile, Error, "Profiles contain invalid name : %ls is reserved for internal use", *CustomCollisionProfileName.ToString());
			Profiles.RemoveAt(Iter.GetIndex());
		}
	}
	// 1. First loads all meta data for custom channels
	// this can be used in #2, so had to fix up first

	// this is to replace ECollisionChannel's DisplayName to be defined by users
	FString EngineTraceChannel	= "EngineTraceChannel";
	FString GameTraceChannel	= "GameTraceChannel";

	// find the enum
	UEnum* Enum = StaticEnum<ECollisionChannel>();
	// we need this Enum
	check (Enum);
	UStruct* Struct = FCollisionResponseContainer::StaticStruct(); 
	check (Struct);
	static const FString DisplayNameKey = "DisplayName";
	static const FString ScriptNameKey = "ScriptName";
	static const FString TraceType = "TraceQuery";
	static const FString TraceValue = "1";
	static const FString HiddenMeta = "Hidden";

	// need to initialize displaynames separate
	int32 NumEnum = Enum->NumEnums();
	ChannelDisplayNames.Empty(NumEnum);
	ChannelDisplayNames.AddZeroed(NumEnum);
	TraceTypeMapping.Empty();
	ObjectTypeMapping.Empty();

	// first go through enum entry, and add suffix to displaynames
	static const FString Prefix = "ECC_";

	auto DisplayNameToScriptName = 
		[](const FString& DisplayName, FStringView DeprecatedScriptName = FStringView())
		{
			FString ScriptName = DisplayName;
			ScriptName.RemoveSpacesInline();
			if (!ScriptName.StartsWith(Prefix))
			{
				ScriptName.InsertAt(0, Prefix);
			}
			if (!DeprecatedScriptName.IsEmpty())
			{
				ScriptName += TEXT(';');
				ScriptName += DeprecatedScriptName;
			}
			return ScriptName;
		};

	// need to have mapping table between ECollisionChannel and EObjectTypeQuery/ETraceTypeQuery
	UEnum* ObjectTypeEnum = StaticEnum<EObjectTypeQuery>();
	UEnum* TraceTypeEnum = StaticEnum<ETraceTypeQuery>();
	check (ObjectTypeEnum && TraceTypeEnum);

	for ( int32 EnumIndex=0; EnumIndex<NumEnum; ++EnumIndex )
	{
		FString EnumName = Enum->GetNameStringByIndex(EnumIndex);
		EnumName.RightChopInline(Prefix.Len(), EAllowShrinking::No);
		FName DisplayName = FName(*EnumName);

		if ( IS_VALID_COLLISIONCHANNEL(EnumIndex) )
		{
			// verify if the Struct name matches
			// this is to avoid situations where they mismatch and causes random bugs
			FField* Field = FindFProperty<FField>(Struct, DisplayName);

			if (!Field)
			{
				// error - this is bad. This means somebody changed variable name without changing channel enum
				UE_LOGF(LogCollisionProfile, Error, "Variable (%ls) isn't found for Channel (%ls). \nPlease make sure you name matches between ECollisionChannel and FCollisionResponseContainer.", *DisplayName.ToString(), *EnumName);
			}

#if WITH_EDITOR
			// clear displayname since we're setting it in below
			Enum->RemoveMetaData(*DisplayNameKey, EnumIndex);
			Enum->RemoveMetaData(*ScriptNameKey, EnumIndex);
			if (Enum->HasMetaData(*HiddenMeta, EnumIndex) == false)
			{
				Enum->SetMetaData(*HiddenMeta, NULL, EnumIndex);
			}
#endif			
		}
		else
		{
			// fix up FCollisionQueryFlag AllObjects flag, if trace type=1
			ECollisionChannel CollisionChannel = (ECollisionChannel)EnumIndex;
			// for any engine level collision profile, we hard coded here
			// meta data doesn't work in cooked build, so we'll have to manually handle them
			if ((CollisionChannel == ECC_Visibility) || (CollisionChannel == ECC_Camera))
			{
				// remove from object query flags
				FCollisionQueryFlag::Get().RemoveFromAllObjectsQueryFlag(CollisionChannel);
				TraceTypeMapping.Add(CollisionChannel);
			}
			else if ( CollisionChannel < ECC_OverlapAll_Deprecated )
			{
				ObjectTypeMapping.Add(CollisionChannel);
			}
		}

		ChannelDisplayNames[EnumIndex] = DisplayName;
	}

	// Now Load Channel setups, and set display names if customized
	// also initialize DefaultResposneContainer with default response for each channel
	FCollisionResponseContainer::DefaultResponseContainer.SetAllChannels(ECR_Block);
	FCollisionResponseContainer::DefaultResponseContainer.SetResponse( COLLISION_GIZMO, ECR_Ignore );

	// we can't guarantee that DefaultChannelResponses was loaded ordered (from
	// config) - in the following loop, we fill out TraceTypeMapping and 
	// ObjectTypeMapping from DefaultChannelResponses, and those mappings expect 
	// to be ordered (since we reinterpret the index to be an enum value itself 
	// in functions like ConvertToCollisionChannel(), etc.)
	DefaultChannelResponses.Sort([](const FCustomChannelSetup& Rhs, const FCustomChannelSetup& Lhs)->bool 
		{ 
			return (Rhs.Channel < Lhs.Channel);
		}
	);

	for (int32 ChannelResponseIndex = 0; ChannelResponseIndex < DefaultChannelResponses.Num(); ++ChannelResponseIndex)
	{
		const FCustomChannelSetup& CustomChannel = DefaultChannelResponses[ChannelResponseIndex];
		int32 EnumIndex = CustomChannel.Channel;
		// make sure it is the range of channels we allow to change
		if ( IS_VALID_COLLISIONCHANNEL(EnumIndex) )
		{
			if ( CustomChannel.Name != NAME_None )
			{
				// before you set meta data, you'll have to save this value to modify 
				// ECollisionResponseContainer variables meta data
				FString VariableName = ChannelDisplayNames[EnumIndex].ToString();
				FString DisplayValue = CustomChannel.Name.ToString();

				if (TraceTypeMapping.Contains(EnumIndex) || ObjectTypeMapping.Contains(EnumIndex))
				{
					UE_LOGF(LogCollisionProfile, Warning, "Cannot map multiple responses to the same collision channel (%d); ignoring '%ls' ", EnumIndex, *DisplayValue);
					DefaultChannelResponses.RemoveAt(ChannelResponseIndex);
					// decrement iterator so this index gets processed again (since we just removed an entry)
					--ChannelResponseIndex;
					continue;
				}

				// also has to set this for internal use
				ChannelDisplayNames[EnumIndex] = FName(*DisplayValue);

#if WITH_EDITOR
				// Create the name to use when exposing this channel for scripting
				FString ScriptValue = DisplayNameToScriptName(DisplayValue);

				// set displayvalue for this enum entry
				Enum->SetMetaData(*DisplayNameKey, *DisplayValue, EnumIndex);
				Enum->SetMetaData(*ScriptNameKey, *ScriptValue, EnumIndex);
				// also need to remove "Hidden"
				Enum->RemoveMetaData(*HiddenMeta, EnumIndex);
#endif 
				// now add MetaData type for trace type if it does
				if (CustomChannel.bTraceType)
				{
#if WITH_EDITOR
					// add to enum
					Enum->SetMetaData(*TraceType, *TraceValue, EnumIndex);
#endif
					// remove from all object queries
					FCollisionQueryFlag::Get().RemoveFromAllObjectsQueryFlag(CustomChannel.Channel);
					TraceTypeMapping.Add(CustomChannel.Channel);
				}
				// if it has display value
				else 
				{
#if WITH_EDITOR
					// add to enum
					Enum->RemoveMetaData(*TraceType, EnumIndex);
#endif
					ObjectTypeMapping.Add(CustomChannel.Channel);

					if (CustomChannel.bStaticObject)
					{
						// add to static object 
						FCollisionQueryFlag::Get().AddToAllStaticObjectsQueryFlag(CustomChannel.Channel);
					}
				}
#if WITH_EDITOR
				// now enum is fixed, so find member variable for the field
				FField* Field = FindFProperty<FField>(Struct, FName(*VariableName));
				// I verified up in the class, this can't happen
				check (Field);
				Field->SetMetaData(*DisplayNameKey, *DisplayValue);
				Field->SetMetaData(*ScriptNameKey, *ScriptValue);
#endif
			}
			else
			{
				// missing name
				UE_LOGF(LogCollisionProfile, Warning, "Name can't be empty for Channel (%d) ", EnumIndex );
			}

			// allow it to set default response
			FCollisionResponseContainer::DefaultResponseContainer.SetResponse((ECollisionChannel) EnumIndex, CustomChannel.DefaultResponse);
		}
		else
		{
			// you can't customize those channels
			UE_LOGF(LogCollisionProfile, Warning, "Default Setup doesn't allow for predefined engine channels (%d) ", EnumIndex);
		}
	}

#if WITH_EDITOR
	// now propagate all changes to the channels to EObjectTypeQuery and ETraceTypeQuery for blueprint accessibility
	// this code is to show in editor more friendly, so it doesn't have to be set if it's not for editor
	int32 ObjectTypeMappingCount = 0;
	int32 TraceTypeMappingCount  = 0;

	// first go through fill up ObjectType Enum
	for ( int32 EnumIndex=0; EnumIndex<NumEnum; ++EnumIndex )
	{
		// if not hidden
		const FString& Hidden = Enum->GetMetaData(*HiddenMeta, EnumIndex);
		if ( Hidden.IsEmpty() )
		{
			const FString& DisplayName = Enum->GetMetaData(*DisplayNameKey, EnumIndex);
			const FString& ScriptName = Enum->GetMetaData(*ScriptNameKey, EnumIndex);
			if ( !DisplayName.IsEmpty() )
			{
				// find out trace type or object type
				if (Enum->GetMetaData(*TraceType, EnumIndex) == TraceValue)
				{
					int32 TraceTypeIndex = TraceTypeMapping.Find((ECollisionChannel)EnumIndex);
					if (ensure(TraceTypeIndex != INDEX_NONE))
					{
						TraceTypeEnum->RemoveMetaData(*HiddenMeta, TraceTypeIndex);
						TraceTypeEnum->SetMetaData(*DisplayNameKey, *DisplayName, TraceTypeIndex);
						TraceTypeEnum->SetMetaData(*ScriptNameKey, ScriptName.IsEmpty() ? *DisplayNameToScriptName(DisplayName, TraceTypeEnum->GetNameStringByIndex(TraceTypeIndex)) : *ScriptName, TraceTypeIndex);
						++TraceTypeMappingCount;
					}
				}
				else
				{
					int32 ObjectTypeIndex = ObjectTypeMapping.Find((ECollisionChannel)EnumIndex);
					if (ensure(ObjectTypeIndex != INDEX_NONE))
					{
						ObjectTypeEnum->RemoveMetaData(*HiddenMeta, ObjectTypeIndex);
						ObjectTypeEnum->SetMetaData(*DisplayNameKey, *DisplayName, ObjectTypeIndex);
						ObjectTypeEnum->SetMetaData(*ScriptNameKey, ScriptName.IsEmpty() ? *DisplayNameToScriptName(DisplayName, ObjectTypeEnum->GetNameStringByIndex(ObjectTypeIndex)) : *ScriptName, ObjectTypeIndex);
						++ObjectTypeMappingCount;
					}
				}
			}
		}
	}

	// make sure TraceTypeEnumIndex matches TraceTypeMapping
	check (TraceTypeMapping.Num()  == TraceTypeMappingCount);
	check (ObjectTypeMapping.Num() == ObjectTypeMappingCount);
#endif

	// collision redirector has to be loaded before profile
	CollisionChannelRedirectsMap.Empty();

	for(auto Iter = CollisionChannelRedirects.CreateConstIterator(); Iter; ++Iter)
	{
		FName OldName = Iter->OldName;
		FName NewName = Iter->NewName;

		// at least we need to have OldName
		if(OldName!= NAME_None && NewName != NAME_None)
		{
			// add to pair
			CollisionChannelRedirectsMap.Add(OldName, NewName);
		}
		else
		{
			// print error 
			UE_LOGF(LogCollisionProfile, Warning, "CollisionChannel Redirects : Name Can't be none (%ls: %ls)", *OldName.ToString(), *NewName.ToString());
		}
	}

	// 2. Second loads all set up back to ResponseToChannels
	// this does a lot of iteration, but this only happens once loaded, so it's better to be convenient than efficient
	// fill up Profiles data
	FillProfileData(Profiles, Enum, EditProfiles);

	// 3. It loads redirect data  - now time to load profile redirect
	ProfileRedirectsMap.Empty();

	// handle profile redirect here
	for(auto Iter = ProfileRedirects.CreateConstIterator(); Iter; ++Iter)
	{
		FName OldName = Iter->OldName;
		FName NewName = Iter->NewName;

		// at least we need to have OldName
		if(OldName!= NAME_None && NewName != NAME_None)
		{
			FCollisionResponseTemplate Template;

			// make sure the template exists
			if(FindProfileData(Profiles, NewName, Template))
			{
				// add to pair
				ProfileRedirectsMap.Add(OldName, NewName);
			}
			else
			{
				// print error
				UE_LOGF(LogCollisionProfile, Warning, "ProfileRedirect (%ls : %ls) - New Name (\'%ls\') isn't found ",
					*OldName.ToString(), *NewName.ToString(), *NewName.ToString());
			}
		}
	}

#if WITH_EDITOR
	if (bForceInit)
	{
		// go through objects, and see if we can reinitialize profile
		for(TObjectIterator<UPrimitiveComponent> PrimIt; PrimIt; ++PrimIt)
		{
			PrimIt->UpdateCollisionProfile();
		}
	}
#endif
}

void UCollisionProfile::FillProfileData(TArray<FCollisionResponseTemplate>& ProfileList, const UEnum* CollisionChannelEnum, TArray<FCustomProfile>& EditProfileList)
{
	// first go through if same name is found later, delete previous entry, this way if game overrides, 
	// we delete engine version and use game version
	int32 NumProfile = ProfileList.Num();
	for ( int32 PriIndex=NumProfile-1; PriIndex>=0 ; --PriIndex )
	{
		FCollisionResponseTemplate& Template = ProfileList[PriIndex];
		// now we iterate previous list
		for ( int32 SubIndex=PriIndex-1; SubIndex>=0; --SubIndex )
		{
			FCollisionResponseTemplate& SubTemplate = ProfileList[SubIndex];
			// if name is same
			if (Template.Name != NAME_None && Template.Name == SubTemplate.Name)
			{
				// delete SubTemplate
				ProfileList.RemoveAt(SubIndex);
				// now you need to decrease all index, otherwise you'll get messed up
				// you're iterating through same list
				--PriIndex;
				--SubIndex;
			}
		}
	}
	// --------------------------------------------------------------------------
	// this is a bit convoluted, but to make editing easier this seems best way
	// my intention here is really to allow users to edit easier
	// First using "DisplayName" will be more user-friendly
	// Second, allowing default reponse simplifies a lot of things
	// --------------------------------------------------------------------------
	for (int32 ProfileIndex = 0; ProfileIndex<ProfileList.Num(); ++ProfileIndex)
	{
		FCollisionResponseTemplate& Template = ProfileList[ProfileIndex];

		if (Template.ObjectTypeName!=NAME_None)
		{
			// find what object type it is and fill it up
			int32 EnumIndex = ReturnContainerIndexFromChannelName(Template.ObjectTypeName);
			if (EnumIndex != INDEX_NONE)
			{
				// first verify if this is real object type
				ECollisionChannel ObjectTypeEnum = (ECollisionChannel)EnumIndex;
				EObjectTypeQuery ObjectTypeQuery = ConvertToObjectType(ObjectTypeEnum);
				if (ObjectTypeQuery != ObjectTypeQuery_MAX)
				{
					Template.ObjectType = ObjectTypeEnum;
				}
				else
				{
					UE_LOGF(LogCollisionProfile, Warning, "Profile (%ls) ObjectTypeName (%ls) is Trace Type. You can set Object Type Channel to Object Type.",
						*Template.Name.ToString(), *Template.ObjectTypeName.ToString());

					ProfileList.RemoveAt(ProfileIndex);
					--ProfileIndex;
					continue;
				}
			}
			else
			{
				UE_LOGF(LogCollisionProfile, Warning, "Profile (%ls) ObjectTypeName (%ls) is invalid. ", 
					*Template.Name.ToString(), *Template.ObjectTypeName.ToString());

				ProfileList.RemoveAt(ProfileIndex);
				--ProfileIndex;
				continue;
			}
		}

		// set to engine default
		Template.ResponseToChannels = FCollisionResponseContainer::DefaultResponseContainer;

		LoadCustomResponses(Template, CollisionChannelEnum, Template.CustomResponses);
		//---------------------------------------------------------------------------------------
		// now load EditProfiles for extra edition of profiles for each game project
		// this one, only search the profile and see if it can change values
		//---------------------------------------------------------------------------------------
		for (auto CustomIter = EditProfileList.CreateIterator(); CustomIter; ++CustomIter)
		{
			if ( (*CustomIter).Name == Template.Name )
			{
				LoadCustomResponses(Template, CollisionChannelEnum, (*CustomIter).CustomResponses);
				break;
			}
		}
	}
}

int32 UCollisionProfile::LoadCustomResponses(FCollisionResponseTemplate& Template, const UEnum* CollisionChannelEnum, TArray<FResponseChannel>& CustomResponses) const
{
	int32 NumOfItemsCustomized=0;

	// now loads all custom setups
	for (auto ChannelIter = CustomResponses.CreateIterator(); ChannelIter; ++ChannelIter)
	{
		FResponseChannel& Custom = *ChannelIter;
		bool bValueFound=false;

		int32 EnumIndex = ReturnContainerIndexFromChannelName(Custom.Channel);
		if (EnumIndex != INDEX_NONE)
		{
			// use the enum index to set to response to channel
			Template.ResponseToChannels.EnumArray[EnumIndex] = Custom.Response;
			bValueFound = true;
			++NumOfItemsCustomized;
		}
		else
		{
			// print error
			UE_LOGF(LogCollisionProfile, Warning, "Profile (%ls) - Custom Channel Name = \'%ls\' hasn't been found", 
				*Template.Name.ToString(), *Custom.Channel.ToString());
		}
	}

	return (NumOfItemsCustomized == CustomResponses.Num());
}

void UCollisionProfile::SaveCustomResponses(FCollisionResponseTemplate& Template) const
{
	const FCollisionResponseContainer& DefaultContainer = FCollisionResponseContainer::GetDefaultResponseContainer();

	Template.CustomResponses.Empty();
	for(int32 Index=0; Index<MaxNumCollisionChannels; ++Index)
	{
		//Save responses different from default
		if(Template.ResponseToChannels.EnumArray[Index] != DefaultContainer.EnumArray[Index])
		{
			const FName ChannelDisplayName = ChannelDisplayNames[Index];
			//The channel should either be a public engine channel or an existing game channel
			if((Index < ECollisionChannel::ECC_EngineTraceChannel1)
				|| (DefaultChannelResponses.FindByPredicate([ChannelDisplayName](const FCustomChannelSetup& ChannelSetup) { return ChannelSetup.Name == ChannelDisplayName; }) != nullptr))
			{
				Template.CustomResponses.Add(FResponseChannel(ChannelDisplayName, (ECollisionResponse)(Template.ResponseToChannels.EnumArray[Index])));
			}
		}
	}
}

int32 UCollisionProfile::ReturnContainerIndexFromChannelName(FName& DisplayName)  const
{
	// if we don't find it in new name
	// @note: I think we can search redirect first in case anybody would like to reuse the name
	// but that seems overhead moving forward. However that is possibility. 
	// this code is only one that has to support old redirects
	// other code should only use new names
	int32 NameIndex = ChannelDisplayNames.Find(DisplayName);
	if (NameIndex == INDEX_NONE)
	{
		// search for redirects
		const FName* NewName = CollisionChannelRedirectsMap.Find(DisplayName);
		if (NewName)
		{
			DisplayName = *NewName;
			return ChannelDisplayNames.Find(*NewName);
		}
	}

	return NameIndex;
}

FName UCollisionProfile::ReturnChannelNameFromContainerIndex(int32 ContainerIndex) const
{
	if (ChannelDisplayNames.IsValidIndex(ContainerIndex))
	{
		return ChannelDisplayNames[ContainerIndex];
	}

	return NAME_None;
}

ECollisionChannel UCollisionProfile::ConvertToCollisionChannel(bool TraceType, int32 Index) const
{
	if (TraceType)
	{
		if ( TraceTypeMapping.IsValidIndex(Index) )
		{
			return TraceTypeMapping[Index];
		}
	}
	else
	{
		if ( ObjectTypeMapping.IsValidIndex(Index) )
		{
			return ObjectTypeMapping[Index];
		}
	}

	// invalid
	return ECC_MAX;
}

EObjectTypeQuery UCollisionProfile::ConvertToObjectType(ECollisionChannel CollisionChannel) const
{
	if (CollisionChannel < ECC_MAX)
	{
		int32 ObjectTypeIndex = 0;
		for(const auto& MappedCollisionChannel : ObjectTypeMapping)
		{
			if(MappedCollisionChannel == CollisionChannel)
			{
				return (EObjectTypeQuery)ObjectTypeIndex;
			}

			ObjectTypeIndex++;
		}
	}

	return ObjectTypeQuery_MAX;
}

ETraceTypeQuery UCollisionProfile::ConvertToTraceType(ECollisionChannel CollisionChannel) const
{
	if (CollisionChannel < ECC_MAX)
	{
		int32 TraceTypeIndex = 0;
		for(const auto& MappedCollisionChannel : TraceTypeMapping)
		{
			if(MappedCollisionChannel == CollisionChannel)
			{
				return (ETraceTypeQuery)TraceTypeIndex;
			}

			TraceTypeIndex++;
		}
	}

	return TraceTypeQuery_MAX;
}

#if WITH_EDITOR
void UCollisionProfile::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	LoadProfileConfig(false);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

bool FCollisionProfilePrivateAccessor::AddProfileTemplate(FCollisionResponseTemplate& NewProfileData)
{
	UCollisionProfile* CollisionProfile = UCollisionProfile::Get();
	FCollisionResponseTemplate ProfileData;
	if (!CollisionProfile->GetProfileTemplate(NewProfileData.Name, ProfileData))
	{
		CollisionProfile->SaveCustomResponses(NewProfileData);
		CollisionProfile->Profiles.Add(NewProfileData);
		if (CollisionProfile->GetProfileTemplate(NewProfileData.Name, ProfileData))
		{
			CollisionProfile->LoadProfileConfig(true);
			CollisionProfile->TryUpdateDefaultConfigFile();
			return true;
		}
	}
	return false;
}

