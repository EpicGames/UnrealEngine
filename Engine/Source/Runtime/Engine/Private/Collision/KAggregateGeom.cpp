// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysCollision.cpp: Skeletal mesh collision code
=============================================================================*/ 

#include "EngineDefines.h"
#include "EngineLogs.h"
#include "Math/BoxSphereBounds.h"
#include "PhysicsEngine/ShapeElem.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/LevelSetElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/SkinnedTriangleMeshElem.h"
#include "PhysicsEngine/TaperedCapsuleElem.h"
#include "Engine/Polys.h"
#include "Chaos/Convex.h"
#include "Chaos/Levelset.h"
#include "Chaos/MLLevelset.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "AutoRTFM.h"
#include "UObject/Package.h"

#if INTEL_ISPC
#include "KAggregateGeom.ispc.generated.h"
#endif

bool bEnableMLLevelSet = false;
static FAutoConsoleVariableRef CVarShowExperimentalImportMLLevelSetButton(TEXT("p.Experimental.EnableMLLevelSet"), bEnableMLLevelSet, TEXT("Shows Import MLLevelSet button. [def: false]"));

#define MIN_HULL_VERT_DISTANCE		(0.1f)
#define MIN_HULL_VALID_DIMENSION	(0.5f)

#if INTEL_ISPC
static_assert(sizeof(ispc::FBox) == sizeof(FBox), "sizeof(ispc::FBox) != sizeof(FBox)");
static_assert(sizeof(ispc::FRotator) == sizeof(FRotator), "sizeof(ispc::FRotator) != sizeof(FRotator)");
static_assert(sizeof(ispc::FVector) == sizeof(FVector), "sizeof(ispc::FVector) != sizeof(FVector)");
#endif

#if !defined(PHYSICS_AGGREGATE_GEOMETRY_ISPC_ENABLED_DEFAULT)
#define PHYSICS_AGGREGATE_GEOMETRY_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bPhysics_AggregateGeom_ISPC_Enabled = INTEL_ISPC && PHYSICS_AGGREGATE_GEOMETRY_ISPC_ENABLED_DEFAULT;
#else
static bool bPhysics_AggregateGeom_ISPC_Enabled = PHYSICS_AGGREGATE_GEOMETRY_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarPhysicsAggregateGeomISPCEnabled(TEXT("p.AggregateGeom.ISPC"), bPhysics_AggregateGeom_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in physics aggregate geometry calculations"));
#endif

///////////////////////////////////////
/////////// FKAggregateGeom ///////////
///////////////////////////////////////

float SelectMinScale(FVector3f Scale)
{
	float Min = Scale.X, AbsMin = FPlatformMath::Abs(Scale.X);

	float  CompareAbsMin  = FPlatformMath::Abs(Scale.Y);
	if (  CompareAbsMin < AbsMin )
	{
		AbsMin = CompareAbsMin;
		Min = Scale.Y;
	}

	CompareAbsMin  = FPlatformMath::Abs(Scale.Z);
	if (  CompareAbsMin < AbsMin )
	{
		AbsMin = CompareAbsMin;
		Min = Scale.Z;
	}

	return Min;
}

FBox FKAggregateGeom::CalcAABB(const FTransform& Transform) const
{
	const FVector3f Scale3D = (FVector3f)Transform.GetScale3D();
	FTransform BoneTM = Transform;
	BoneTM.RemoveScaling();

	FBox Box(ForceInit);

	// Instead of ignore if not uniform, I'm getting Min of the abs value
	// the reason for below function is for negative scale
	// say if you have scale of (-1, 2, -3), you'd like to get -1;
	const float ScaleFactor = SelectMinScale(Scale3D);

	for(int32 i=0; i<SphereElems.Num(); i++)
	{
		Box += SphereElems[i].CalcAABB(BoneTM, ScaleFactor);
	}

	for(int32 i=0; i<BoxElems.Num(); i++)
	{
		Box += BoxElems[i].CalcAABB(BoneTM, ScaleFactor);
	}

	for(int32 i=0; i<SphylElems.Num(); i++)
	{
		Box += SphylElems[i].CalcAABB(BoneTM, ScaleFactor);
	}

	// Accumulate convex element bounding boxes.
	for(int32 i=0; i<ConvexElems.Num(); i++)
	{
		Box += ConvexElems[i].CalcAABB(BoneTM, (FVector)Scale3D);
	}

	for(int32 i=0; i<TaperedCapsuleElems.Num(); i++)
	{
		Box += TaperedCapsuleElems[i].CalcAABB(BoneTM, ScaleFactor);
	}

	for (int32 i = 0; i < LevelSetElems.Num(); i++)
	{
		Box += LevelSetElems[i].CalcAABB(BoneTM, (FVector)Scale3D);
	}

	for (int32 i = 0; i < SkinnedLevelSetElems.Num(); i++)
	{
		Box += SkinnedLevelSetElems[i].CalcAABB(BoneTM, (FVector)Scale3D);
	}

	for (int32 i = 0; i < MLLevelSetElems.Num(); i++)
	{
		Box += MLLevelSetElems[i].CalcAABB(BoneTM, (FVector)Scale3D);
	}

	for (int32 i = 0; i < SkinnedTriangleMeshElems.Num(); i++)
	{
		Box += SkinnedTriangleMeshElems[i].CalcAABB(BoneTM, (FVector)Scale3D);
	}

	return Box;

}

/**
  * Calculates a tight box-sphere bounds for the aggregate geometry; this is more expensive than CalcAABB
  * (tight meaning the sphere may be smaller than would be required to encompass the AABB, but all individual components lie within both the box and the sphere)
  *
  * @param Output The output box-sphere bounds calculated for this set of aggregate geometry
  *	@param LocalToWorld Transform
  */
void FKAggregateGeom::CalcBoxSphereBounds(FBoxSphereBounds& Output, const FTransform& LocalToWorld) const
{
	// Calculate the AABB
	const FBox AABB = CalcAABB(LocalToWorld);

	if ((SphereElems.Num() == 0) && (SphylElems.Num() == 0) && (BoxElems.Num() == 0))
	{
		// For bounds that only consist of convex shapes (such as anything generated from a BSP model),
		// we can get nice tight bounds by considering just the points of the convex shape
		const FVector3f Origin = (FVector3f)AABB.GetCenter();

		float RadiusSquared = 0.0f;
		for (int32 i = 0; i < ConvexElems.Num(); i++)
		{
			const FKConvexElem& Elem = ConvexElems[i];
			for (int32 j = 0; j < Elem.VertexData.Num(); ++j)
			{
				const FVector3f Point = (FVector3f)LocalToWorld.TransformPosition(Elem.VertexData[j]);
				RadiusSquared = FMath::Max(RadiusSquared, (Point - Origin).SizeSquared());
			}
		}

		// Push the resulting AABB and sphere into the output
		AABB.GetCenterAndExtents(Output.Origin, Output.BoxExtent);
		Output.SphereRadius = FMath::Sqrt(RadiusSquared);
	}
	else if ((SphereElems.Num() == 1) && (SphylElems.Num() == 0) && (BoxElems.Num() == 0) && (ConvexElems.Num() == 0))
	{
		// For bounds that only consist of a single sphere,
		// we can be certain the box extents are the same as its radius
		AABB.GetCenterAndExtents(Output.Origin, Output.BoxExtent);
		Output.SphereRadius = Output.BoxExtent.X;
	}
	else
	{
		// Just use the loose sphere bounds that totally fit the AABB
		Output = FBoxSphereBounds(AABB);
	}
}


static int32 AddVertexIfNotPresent(TArray<FVector> &Vertices, const FVector& NewVertex)
{
	bool bIsPresent = 0;
	int32 Index = INDEX_NONE;

	for(int32 i = 0; i < Vertices.Num() && !bIsPresent; i++)
	{
		const float DiffSqr = (NewVertex - Vertices[i]).SizeSquared();
		if(DiffSqr < MIN_HULL_VERT_DISTANCE * MIN_HULL_VERT_DISTANCE)
		{
			bIsPresent = true;
			Index = i;
		}
	}

	if(!bIsPresent)
	{
		Vertices.Add(NewVertex);
		Index = Vertices.Num() - 1;
	}

	return Index;
}

static void RemoveDuplicateVerts(TArray<FVector>& InVerts)
{
	TArray<FVector> BackupVerts = InVerts;
	InVerts.Empty();

	for(int32 i=0; i<BackupVerts.Num(); i++)
	{
		AddVertexIfNotPresent(InVerts, BackupVerts[i]);
	}
}

// Weisstein, Eric W. "Point-Line Distance--3-Dimensional." From MathWorld--A Switchram Web Resource. http://mathworld.wolfram.com/Point-LineDistance3-Dimensional.html 
static float DistanceToLine(const FVector3f& LineStart, const FVector3f& LineEnd, const FVector3f& Point)
{
	const FVector3f StartToEnd = LineEnd - LineStart;
	const FVector3f PointToStart = LineStart - Point;

	const FVector3f Cross = StartToEnd ^ PointToStart;
	return Cross.Size()/StartToEnd.Size();
}

/** 
 *	Utility that ensures the verts supplied form a valid hull. Will modify the verts to remove any duplicates. 
 *	Positions should be in physics scale.
 *	Returns true is hull is valid.
 */
static bool EnsureHullIsValid(TArray<FVector>& InVerts)
{
	RemoveDuplicateVerts(InVerts);

	if(InVerts.Num() < 3)
	{
		return false;
	}

	// Take any vert. In this case - the first one.
	const FVector FirstVert = InVerts[0];

	// Now find vert furthest from this one.
	float FurthestDistSqr = 0.f;
	int32 FurthestVertIndex = INDEX_NONE;
	for(int32 i=1; i<InVerts.Num(); i++)
	{
		const float TestDistSqr = (InVerts[i] - FirstVert).SizeSquared();
		if(TestDistSqr > FurthestDistSqr)
		{
			FurthestDistSqr = TestDistSqr;
			FurthestVertIndex = i;
		}
	}

	// If smallest dimension is too small - hull is invalid.
	if(FurthestVertIndex == INDEX_NONE || FurthestDistSqr < FMath::Square(MIN_HULL_VALID_DIMENSION))
	{
		return false;
	}

	// Now find point furthest from line defined by these 2 points.
	float ThirdPointDist = 0.f;
	int32 ThirdPointIndex = INDEX_NONE;
	for(int32 i=1; i<InVerts.Num(); i++)
	{
		if(i != FurthestVertIndex)
		{
			const float TestDist = DistanceToLine((FVector3f)FirstVert, (FVector3f)InVerts[FurthestVertIndex], (FVector3f)InVerts[i]);
			if(TestDist > ThirdPointDist)
			{
				ThirdPointDist = TestDist;
				ThirdPointIndex = i;
			}
		}
	}

	// If this dimension is too small - hull is invalid.
	if(ThirdPointIndex == INDEX_NONE || ThirdPointDist < MIN_HULL_VALID_DIMENSION)
	{
		return false;
	}

	// Now we check each remaining point against this plane.

	// First find plane normal.
	const FVector Dir1 = InVerts[FurthestVertIndex] - InVerts[0];
	const FVector Dir2 = InVerts[ThirdPointIndex] - InVerts[0];
	FVector PlaneNormal = Dir1 ^ Dir2;
	const bool bNormalizedOk = PlaneNormal.Normalize();
	if(!bNormalizedOk)
	{
		return false;
	}

	// Now iterate over all remaining vertices.
	float MaxThickness = 0.f;
	for(int32 i=1; i<InVerts.Num(); i++)
	{
		if((i != FurthestVertIndex) && (i != ThirdPointIndex))
		{
			const float PointPlaneDist = FMath::Abs((InVerts[i] - InVerts[0]) | PlaneNormal);
			MaxThickness = FMath::Max(PointPlaneDist, MaxThickness);
		}
	}

	if(MaxThickness < MIN_HULL_VALID_DIMENSION)
	{
		return false;
	}

	return true;
}

///////////////////////////////////////
///////////// FKShapeElem ////////////
///////////////////////////////////////

FKShapeElem::~FKShapeElem() = default;

EAggCollisionShape::Type FKShapeElem::StaticShapeType = EAggCollisionShape::Unknown;


///////////////////////////////////////
///////////// FKSphereElem ////////////
///////////////////////////////////////

FKSphereElem::~FKSphereElem() = default;

EAggCollisionShape::Type FKSphereElem::StaticShapeType = EAggCollisionShape::Sphere;

FBox FKSphereElem::CalcAABB(const FTransform& BoneTM, float Scale) const
{
	FTransform ElemTM = GetTransform();
	ElemTM.ScaleTranslation( FVector(Scale) );
	ElemTM *= BoneTM;

	const FVector3f BoxCenter = (FVector3f)ElemTM.GetTranslation();
	const FVector3f BoxExtents(Radius * Scale);

	return FBox(BoxCenter - BoxExtents, BoxCenter + BoxExtents);
}


///////////////////////////////////////
////////////// FKBoxElem //////////////
///////////////////////////////////////

FKBoxElem::~FKBoxElem() = default;

EAggCollisionShape::Type FKBoxElem::StaticShapeType = EAggCollisionShape::Box;

FBox FKBoxElem::CalcAABB(const FTransform& BoneTM, float Scale) const
{
	if (bPhysics_AggregateGeom_ISPC_Enabled)
	{
#if INTEL_ISPC
		FBox LocalBox(ForceInit);
		UE_AUTORTFM_OPEN
		{
			ispc::BoxCalcAABB(
				reinterpret_cast<ispc::FBox&>(LocalBox),
				reinterpret_cast<const ispc::FTransform&>(BoneTM),
				Scale,
				reinterpret_cast<const ispc::FRotator&>(Rotation),
				reinterpret_cast<const ispc::FVector&>(Center),
				X,
				Y,
				Z);
		};
		return LocalBox;
#endif
	}
	else
	{
		FTransform ElemTM = GetTransform();
		ElemTM.ScaleTranslation(FVector(Scale));
		ElemTM *= BoneTM;

		FVector3f Extent(0.5f * Scale * X, 0.5f * Scale * Y, 0.5f * Scale * Z);
		FBox LocalBox(-Extent, Extent);

		return LocalBox.TransformBy(ElemTM);
	}
}


///////////////////////////////////////
////////////// FKSphylElem ////////////
///////////////////////////////////////

FKSphylElem::~FKSphylElem() = default;

EAggCollisionShape::Type FKSphylElem::StaticShapeType = EAggCollisionShape::Sphyl;

FBox FKSphylElem::CalcAABB(const FTransform& BoneTM, float Scale) const
{
	if (bPhysics_AggregateGeom_ISPC_Enabled)
	{
#if INTEL_ISPC
		FBox Result(ForceInit);
		UE_AUTORTFM_OPEN
		{
			ispc::SPhylCalcAABB(
				reinterpret_cast<ispc::FBox&>(Result),
				reinterpret_cast<const ispc::FTransform&>(BoneTM),
				Scale,
				reinterpret_cast<const ispc::FRotator&>(Rotation),
				reinterpret_cast<const ispc::FVector&>(Center),
				Radius,
				Length);
		};
		return Result;
#endif
	}
	else
	{
		FTransform ElemTM = GetTransform();
		ElemTM.ScaleTranslation( FVector(Scale) );
		ElemTM *= BoneTM;

		const FVector3f SphylCenter = (FVector3f)ElemTM.GetLocation();

		// Get sphyl axis direction
		const FVector3f Axis = (FVector3f)ElemTM.GetScaledAxis( EAxis::Z );
		// Get abs of that vector
		const FVector3f AbsAxis(FMath::Abs(Axis.X), FMath::Abs(Axis.Y), FMath::Abs(Axis.Z));
		// Scale by length of sphyl
		const FVector3f AbsDist = (Scale * 0.5f * Length) * AbsAxis;

		const FVector3f MaxPos = SphylCenter + AbsDist;
		const FVector3f MinPos = SphylCenter - AbsDist;
		const FVector3f Extent(Scale * Radius);

		FBox Result(MinPos - Extent, MaxPos + Extent);

		return Result;
	}
}


///////////////////////////////////////
///////////// FKConvexElem ////////////
///////////////////////////////////////

EAggCollisionShape::Type FKConvexElem::StaticShapeType = EAggCollisionShape::Convex;

/** Reset the hull to empty all arrays */
void FKConvexElem::Reset()
{
	VertexData.Empty();
	ElemBox.Init();
}

FBox FKConvexElem::CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const
{
	// Zero out rotation and location so we transform by scale along
	const FTransform LocalToWorld = FTransform(FQuat::Identity, FVector::ZeroVector, Scale3D) * BoneTM;

	return ElemBox.TransformBy(Transform * LocalToWorld);
}

void FKConvexElem::GetPlanes(TArray<FPlane>& Planes) const
{
	using FChaosPlane = Chaos::TPlaneConcrete<Chaos::FReal, 3>;
	if(Chaos::FConvex* RawConvex = ChaosConvex.GetReference())
	{
		const int32 NumPlanes = RawConvex->NumPlanes();
		for(int32 i = 0; i < NumPlanes; ++i)
		{
			const FChaosPlane& Plane = RawConvex->GetPlane(i);

			Planes.Add({Plane.X(), Plane.Normal()});
		}
	}
}

///////////////////////////////////////
//////// FKTaperedCapsuleElem /////////
///////////////////////////////////////

FKTaperedCapsuleElem::FKTaperedCapsuleElem()
: FKShapeElem(EAggCollisionShape::TaperedCapsule)
, Center(FVector::ZeroVector)
, Rotation(FRotator::ZeroRotator)
, Radius0(1.0f)
, Radius1(1.0f)
, Length(1.0f)
{
}

FKTaperedCapsuleElem::FKTaperedCapsuleElem( float InRadius0, float InRadius1, float InLength )
: FKShapeElem(EAggCollisionShape::TaperedCapsule)
, Center(FVector::ZeroVector)
, Rotation(FRotator::ZeroRotator)
, Radius0(InRadius0)
, Radius1(InRadius1)
, Length(InLength)
{
}

FKTaperedCapsuleElem::FKTaperedCapsuleElem(const FKTaperedCapsuleElem&) = default;
FKTaperedCapsuleElem::~FKTaperedCapsuleElem() = default;

EAggCollisionShape::Type FKTaperedCapsuleElem::StaticShapeType = EAggCollisionShape::TaperedCapsule;

FBox FKTaperedCapsuleElem::CalcAABB(const FTransform& BoneTM, float Scale) const
{
	FTransform ElemTM = GetTransform();
	ElemTM.ScaleTranslation( FVector(Scale) );
	ElemTM *= BoneTM;

	const FVector3f TaperedCapsuleCenter = (FVector3f)ElemTM.GetLocation();

	// Get tapered capsule axis direction
	const FVector3f Axis = (FVector3f)ElemTM.GetScaledAxis( EAxis::Z );
	// Get abs of that vector
	const FVector3f AbsAxis(FMath::Abs(Axis.X), FMath::Abs(Axis.Y), FMath::Abs(Axis.Z));
	// Scale by length of sphyl
	const FVector3f AbsDist = (Scale * 0.5f * Length) * AbsAxis;

	// Get extruded tapered capsule axis direction
	const FVector3f ExtrudedAxis = (FVector3f)ElemTM.GetScaledAxis(EAxis::X);
	// Get abs of that vector
	const FVector3f AbsExtrudedAxis(FMath::Abs(ExtrudedAxis.X), FMath::Abs(ExtrudedAxis.Y), FMath::Abs(ExtrudedAxis.Z));
	// Scale by width of sphyl
	const FVector3f AbsExtrudedDist = (Scale * 0.5f * Width) * AbsExtrudedAxis;


	const FVector3f MaxPos = TaperedCapsuleCenter + AbsDist + AbsExtrudedDist;
	const FVector3f MinPos = TaperedCapsuleCenter - AbsDist - AbsExtrudedDist;
	const FVector3f Extent0(Scale * Radius0);
	const FVector3f Extent1(Scale * Radius1);

	FBox Result(MinPos - Extent0, MaxPos + Extent1);

	return Result;
}


///////////////////////////////////////
//////// FKLevelSetElem ///////////////
///////////////////////////////////////

FBox FKLevelSetElem::CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const
{
	FBox Box(ForceInit);

	if (LevelSet.IsValid())
	{
		Box = FBox(LevelSet->BoundingBox().Min(), LevelSet->BoundingBox().Max());
		const FTransform LocalToWorld = FTransform(FQuat::Identity, FVector::ZeroVector, Scale3D) * BoneTM;
		return Box.TransformBy(Transform * LocalToWorld);
	}

	return Box;
}

FBox FKLevelSetElem::UntransformedAABB() const
{
	FBox Box(ForceInit);

	if (LevelSet.IsValid())
	{
		Box = FBox(LevelSet->BoundingBox().Min(), LevelSet->BoundingBox().Max());
		return Box;
	}

	return Box;
}

FIntVector3 FKLevelSetElem::GridResolution() const
{
	if (LevelSet.IsValid())
	{
		const Chaos::TVec3<int32>& Dim = LevelSet->GetGrid().Counts();
		return FIntVector3(Dim[0], Dim[1], Dim[2]);
	}

	return FIntVector3(0,0,0);
}

void FKLevelSetElem::BuildLevelSet(const FTransform& GridTransform, const TArray<double>& GridValues, const FIntVector& GridDims, float GridCellSize)
{
	const Chaos::FVec3 Min(0, 0, 0);
	const Chaos::FVec3 Max = Min + Chaos::FVec3(GridCellSize * (FVector)GridDims);
	const Chaos::TVec3<int32> ChaosDim(GridDims[0], GridDims[1], GridDims[2]);
	
	Chaos::TUniformGrid<Chaos::FReal, 3> ChaosGrid(Min, Max, ChaosDim);
	Chaos::TArrayND<Chaos::FReal, 3> Phi( Chaos::TVec3<int32>(GridDims[0], GridDims[1], GridDims[2]), GridValues );

	LevelSet = TSharedPtr<Chaos::FLevelSet>(new Chaos::FLevelSet(MoveTemp(ChaosGrid), MoveTemp(Phi), 0));

	Transform = GridTransform;
}

void FKLevelSetElem::GetLevelSetData(FTransform& OutGridTransform, TArray<double>& OutGridValues, FIntVector& OutGridDims, float& OutGridCellSize) const
{
	if (!ensure(LevelSet.IsValid()))
	{
		return;
	}
	const Chaos::TUniformGrid<Chaos::FReal, 3>& ChaosGrid = LevelSet->GetGrid();

	OutGridTransform = Transform;
	OutGridDims = FIntVector(ChaosGrid.Counts()[1], ChaosGrid.Counts()[0], ChaosGrid.Counts()[2]);

	OutGridCellSize = ChaosGrid.Dx()[0];

	OutGridValues.Init(0.0, LevelSet->GetPhiArray().Num());
	for (int32 Index = 0; Index < ChaosGrid.GetNumCells(); ++Index)
	{
		OutGridValues[Index] = LevelSet->GetPhiArray()[Index];
	}
}


void FKLevelSetElem::GetInteriorGridCells( TArray<FBox>& CellBoxes, double InteriorThreshold) const
{
	if (LevelSet.IsValid())
	{
		const Chaos::TUniformGrid<Chaos::FReal, 3>& Grid = LevelSet->GetGrid();
		const Chaos::TVector<int32, 3> Cells = Grid.Counts();
		for (int i = 0; i < Cells.X; ++i)
		{
			for (int j = 0; j < Cells.Y; ++j)
			{
				for (int k = 0; k < Cells.Z; ++k)
				{
					const double Value = LevelSet->GetPhiArray()(i, j, k);

					if (Value <= InteriorThreshold)
					{
						const FVector3d CellMin = Grid.MinCorner() + Grid.Dx() * FVector3d(i, j, k);
						const FVector3d CellMax = CellMin + Grid.Dx() * FVector3d(1, 1, 1);
						CellBoxes.Emplace( FBox(CellMin, CellMax) );
					}
				}
			}
		}
	}
}


void FKLevelSetElem::GetZeroIsosurfaceGridCellFaces(TArray<FVector3f>& Vertices, TArray<FIntVector>& Tris ) const
{
	if (LevelSet.IsValid())
	{
		LevelSet->GetZeroIsosurfaceGridCellFaces(Vertices, Tris);
	}
}


///////////////////////////////////////
//////// FKSkinnedLevelSetElem ////////
///////////////////////////////////////

FBox FKSkinnedLevelSetElem::CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const
{
	FBox Box(ForceInit);

	if (WeightedLatticeLevelSet.IsValid())
	{
		Box = FBox(WeightedLatticeLevelSet->BoundingBox().Min(), WeightedLatticeLevelSet->BoundingBox().Max());
		const FTransform LocalToWorld = FTransform(FQuat::Identity, FVector::ZeroVector, Scale3D) * BoneTM;
		return Box.TransformBy(LocalToWorld);
	}

	return Box;
}

FIntVector3 FKSkinnedLevelSetElem::LevelSetGridResolution() const
{
	if (WeightedLatticeLevelSet.IsValid())
	{
		const Chaos::TVec3<int32>& Dim = WeightedLatticeLevelSet->GetEmbeddedObject()->GetGrid().Counts();
		return FIntVector3(Dim[0], Dim[1], Dim[2]);
	}

	return FIntVector3(0, 0, 0);
}

FIntVector3 FKSkinnedLevelSetElem::LatticeGridResolution() const
{
	if (WeightedLatticeLevelSet.IsValid())
	{
		const Chaos::TVec3<int32>& Dim = WeightedLatticeLevelSet->GetGrid().Counts();
		return FIntVector3(Dim[0], Dim[1], Dim[2]);
	}

	return FIntVector3(0, 0, 0);
}

void FKSkinnedLevelSetElem::SetWeightedLevelSet(TRefCountPtr< Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>>&& InWeightedLevelSet)
{
	WeightedLatticeLevelSet = MoveTemp(InWeightedLevelSet);
}

///////////////////////////////////////
//////// FKMLLevelSetElem ///////////////
///////////////////////////////////////


FBox FKMLLevelSetElem::CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const
{
	FBox Box(ForceInit);

	if (MLLevelSet.IsValid())
	{
		Box = FBox(MLLevelSet->BoundingBox().Min(), MLLevelSet->BoundingBox().Max());
		const FTransform LocalToWorld = FTransform(FQuat::Identity, FVector::ZeroVector, Scale3D) * BoneTM;
		return Box.TransformBy(LocalToWorld);
	}

	return Box;
}

FBox FKMLLevelSetElem::UntransformedAABB() const
{
	FBox Box(ForceInit);

	if (MLLevelSet.IsValid())
	{
		Box = FBox(MLLevelSet->BoundingBox().Min(), MLLevelSet->BoundingBox().Max());
		return Box;
	}

	return Box;
}

FIntVector3 FKMLLevelSetElem::GridResolution() const
{
	if (MLLevelSet.IsValid())
	{
		const Chaos::TVec3<int32>& Dim = MLLevelSet->GetGrid().Counts();
		return FIntVector3(Dim[0], Dim[1], Dim[2]);
	}

	return FIntVector3(0, 0, 0);
}

void FKMLLevelSetElem::BuildMLLevelSet(Chaos::FMLLevelSetImportData&& MLLevelSetImportData)
{
	NNESignedDistanceModelData = LoadObject<UNNEModelData>(GetTransientPackage(), *MLLevelSetImportData.NNEModelDataArr[0].NNEModelPath);
	NNEIncorrectZoneModelData = LoadObject<UNNEModelData>(GetTransientPackage(), *MLLevelSetImportData.NNEModelDataArr[1].NNEModelPath);
		
	MLLevelSetImportData.NNEModelDataArr[0].NNEModelData = NNESignedDistanceModelData;
	MLLevelSetImportData.NNEModelDataArr[1].NNEModelData = NNEIncorrectZoneModelData;

	MLLevelSet = TSharedPtr<Chaos::FMLLevelSet>(new Chaos::FMLLevelSet(MoveTemp(MLLevelSetImportData)));
}

void FKMLLevelSetElem::GetInteriorGridCells(TArray<FBox>& CellBoxes, double InteriorThreshold) const
{
	if (MLLevelSet.IsValid())
	{
		const Chaos::TUniformGrid<Chaos::FReal, 3>& Grid = MLLevelSet->GetGrid();
		const Chaos::TVector<int32, 3> Cells = Grid.Counts();
		for (int i = 0; i < Cells.X; ++i)
		{
			for (int j = 0; j < Cells.Y; ++j)
			{
				for (int k = 0; k < Cells.Z; ++k)
				{
					const double Value = MLLevelSet->GetPhiArray()(i, j, k);

					if (Value <= InteriorThreshold)
					{
						const FVector3d CellMin = Grid.MinCorner() + Grid.Dx() * FVector3d(i, j, k);
						const FVector3d CellMax = CellMin + Grid.Dx() * FVector3d(1, 1, 1);
						CellBoxes.Emplace(FBox(CellMin, CellMax));
					}
				}
			}
		}
	}
}

void FKMLLevelSetElem::GetZeroIsosurfaceGridCellFaces(TArray<FVector3f>& Vertices, TArray<FIntVector>& Tris) const
{
	if (MLLevelSet.IsValid())
	{
		MLLevelSet->GetZeroIsosurfaceGridCellFaces(Vertices, Tris);
	}
}


///////////////////////////////////////
///// FKSkinnedTriangleMeshElem ///////
///////////////////////////////////////
FBox FKSkinnedTriangleMeshElem::CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const
{
	FBox Box(ForceInit);
	if (SkinnedTriangleMesh.IsValid())
	{
		Box = FBox(SkinnedTriangleMesh->BoundingBox().Min(), SkinnedTriangleMesh->BoundingBox().Max());
		const FTransform LocalToWorld = FTransform(FQuat::Identity, FVector::ZeroVector, Scale3D) * BoneTM;
		return Box.TransformBy(LocalToWorld);
	}

	return Box;
}

void FKSkinnedTriangleMeshElem::SetSkinnedTriangleMesh(TRefCountPtr<Chaos::FSkinnedTriangleMesh>&& InSkinnedTriangleMesh)
{
	SkinnedTriangleMesh = MoveTemp(InSkinnedTriangleMesh);
}

const TRefCountPtr<Chaos::FSkinnedTriangleMesh>& FKSkinnedTriangleMeshElem::GetSkinnedTriangleMesh() const
{
	return SkinnedTriangleMesh;
}

static const float DIST_COMPARE_THRESH = 0.1f;
static const float DIR_COMPARE_THRESH = 0.0003f; // about 1 degree


static bool TriHasEdge(int32 T0, int32 T1, int32 T2, int32 Edge0, int32 Edge1)
{
	if(	(T0 == Edge0 && T1 == Edge1) || (T1 == Edge0 && T0 == Edge1) || 
		(T0 == Edge0 && T2 == Edge1) || (T2 == Edge0 && T0 == Edge1) || 
		(T1 == Edge0 && T2 == Edge1) || (T2 == Edge0 && T1 == Edge1) )
	{
		return true;
	}

	return false;
}

static void GetTriIndicesUsingEdge(const int32 Edge0, const int32 Edge1, const TArray<int32>& TriData, int32& Tri0Index, int32& Tri1Index)
{
	Tri0Index = INDEX_NONE;
	Tri1Index = INDEX_NONE;

	const int32 NumTris = TriData.Num()/3;

	// Iterate over triangles, looking for ones that contain this edge.
	for(int32 i=0; i<NumTris; i++)
	{
		if( TriHasEdge(TriData[(i*3)+0], TriData[(i*3)+1], TriData[(i*3)+2], Edge0, Edge1) )
		{
			if(Tri0Index == INDEX_NONE)
			{
				Tri0Index = i;
			}
			else if(Tri1Index == INDEX_NONE)
			{
				Tri1Index = i;
			}
			else
			{
				UE_LOGF(LogPhysics, Log,  "GetTriIndicesUsingEdge : 3 tris share an edge." );
			}
		}
	}
}

static void AddEdgeIfNotPresent(TArray<int32>& Edges, int32 Edge0, int32 Edge1)
{
	const int32 NumEdges = Edges.Num()/2;

	// See if this edge is already present.
	for(int32 i=0; i<NumEdges; i++)
	{
		int32 TestEdge0 = Edges[(i*2)+0];
		int32 TestEdge1 = Edges[(i*2)+1];

		if( (TestEdge0 == Edge0 && TestEdge1 == Edge1) ||
			(TestEdge1 == Edge0 && TestEdge0 == Edge1) )
		{
			// It is - do nothing
			return;
		}
	}

	Edges.Add(Edge0);
	Edges.Add(Edge1);
}

#define LOCAL_EPS UE_SMALL_NUMBER

void FKConvexElem::UpdateElemBox()
{
	// Fixup indices in case an operation has invalidated them
	{
		IndexData.Reset();
		ComputeChaosConvexIndices();
	}

	ElemBox.Init();
	for(int32 j=0; j<VertexData.Num(); j++)
	{
		ElemBox += VertexData[j];
	}
}

bool FKConvexElem::HullFromPlanes(const TArray<FPlane>& InPlanes, const TArray<FVector>& SnapVerts, float InSnapDistance)
{
	// Start by clearing this convex.
	Reset();

	float TotalPolyArea = 0;

	for(int32 i=0; i<InPlanes.Num(); i++)
	{
		FPoly Polygon;
		Polygon.Normal = (FVector3f)InPlanes[i];

		FVector3f AxisX, AxisY;
		Polygon.Normal.FindBestAxisVectors(AxisX,AxisY);

		const FVector3f Base = FVector3f(InPlanes[i] * InPlanes[i].W);

		new(Polygon.Vertices) FVector3f(Base + AxisX * UE_OLD_HALF_WORLD_MAX + AxisY * UE_OLD_HALF_WORLD_MAX);
		new(Polygon.Vertices) FVector3f(Base - AxisX * UE_OLD_HALF_WORLD_MAX + AxisY * UE_OLD_HALF_WORLD_MAX);
		new(Polygon.Vertices) FVector3f(Base - AxisX * UE_OLD_HALF_WORLD_MAX - AxisY * UE_OLD_HALF_WORLD_MAX);
		new(Polygon.Vertices) FVector3f(Base + AxisX * UE_OLD_HALF_WORLD_MAX - AxisY * UE_OLD_HALF_WORLD_MAX);

		for(int32 j=0; j<InPlanes.Num(); j++)
		{
			if(i != j)
			{
				if(!Polygon.Split(-FVector3f(InPlanes[j]), FVector3f(InPlanes[j] * InPlanes[j].W)))
				{
					Polygon.Vertices.Empty();
					break;
				}
			}
		}

		// Do nothing if poly was completely clipped away.
		if(Polygon.Vertices.Num() > 0)
		{
			TArray<int32> Remap;
			Remap.AddUninitialized(Polygon.Vertices.Num());

			TotalPolyArea += Polygon.Area();

			// Add vertices of polygon to convex primitive.
			for(int32 j = 0; j < Polygon.Vertices.Num() ; j++)
			{
				// We try and snap the vert to on of the ones supplied.
				int32 NearestVert = INDEX_NONE;
				float NearestDistSqr = UE_BIG_NUMBER;

				for(int32 k = 0; k < SnapVerts.Num(); k++)
				{
					const float DistSquared = ((FVector)Polygon.Vertices[j] - (FVector)SnapVerts[k]).SizeSquared();

					if( DistSquared < NearestDistSqr )
					{
						NearestVert = k;
						NearestDistSqr = DistSquared;
					}
				}

				// If we have found a suitably close vertex, use that
				if( NearestVert != INDEX_NONE && NearestDistSqr < InSnapDistance)
				{
					const FVector localVert = SnapVerts[NearestVert];
					Remap[j] = AddVertexIfNotPresent(VertexData, localVert);
				}
				else
				{
					const FVector localVert = (FVector)Polygon.Vertices[j];
					Remap[j] = AddVertexIfNotPresent(VertexData, localVert);
				}
			}

			const int32 NumTriangles = Polygon.Vertices.Num() - 2;
			const int32 BaseIndex = Remap[0];
			for(int32 Index = 0; Index < NumTriangles; ++Index)
			{
				IndexData.Add(BaseIndex);
				IndexData.Add(Remap[Index + 1]);
				IndexData.Add(Remap[Index + 2]);
			}
		}
	}

	// If the collision volume isn't closed, return an error so the model can be discarded
	if(TotalPolyArea < 0.001f)
	{
		UE_LOGF(LogPhysics, Log,  "Total Polygon Area invalid: %f", TotalPolyArea );
		return false;
	}

	// We need at least 4 vertices to make a convex hull with non-zero volume.
	// We shouldn't have the same vertex multiple times (using AddVertexIfNotPresent above)
	if(VertexData.Num() < 4)
	{
		return false;
	}

	// Check that not all vertices lie on a line (ie. find plane)
	// Again, this should be a non-zero vector because we shouldn't have duplicate verts.
	bool bFound = false;
	FVector Dir2, Dir1;

	Dir1 = VertexData[1] - VertexData[0];
	Dir1.Normalize();

	for(int32 i=2; i<VertexData.Num() && !bFound; i++)
	{
		Dir2 = VertexData[i] - VertexData[0];
		Dir2.Normalize();

		// If line are non-parallel, this vertex forms our plane
		if((Dir1 | Dir2) < (1.f - LOCAL_EPS))
		{
			bFound = true;
		}
	}

	if(!bFound)
	{
		return false;
	}

	// Now we check that not all vertices lie on a plane, by checking at least one lies off the plane we have formed.
	FVector Normal = Dir1 ^ Dir2;
	Normal.Normalize();

	const FPlane Plane(VertexData[0], Normal);

	bFound = false;
	for(int32 i=2; i<VertexData.Num() ; i++)
	{
		if(FMath::Abs(Plane.PlaneDot(VertexData[i])) > LOCAL_EPS)
		{
			bFound = true;
			break;
		}
	}

	// If we did not find a vert off the line - discard this hull.
	if(!bFound)
	{
		return false;
	}

	// calc bounding box of verts
	UpdateElemBox();

	// We can continue adding primitives (mesh is not horribly broken)
	return true;
}

void FKConvexElem::ConvexFromBoxElem(const FKBoxElem& InBox)
{
	Reset();

	FVector3f	B[2], P, Q, Radii;

	// X,Y,Z member variables are LENGTH not RADIUS
	Radii.X = 0.5f*InBox.X;
	Radii.Y = 0.5f*InBox.Y;
	Radii.Z = 0.5f*InBox.Z;

	B[0] = Radii; // max
	B[1] = -1.0f * Radii; // min

	// copy transform
	Transform = InBox.GetTransform();

	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			const int32 Base = VertexData.Num();
			for(int32 k = 0; k < 6; ++k)
			{
				IndexData.Add(Base + k);
			}

			P.X = B[i].X; Q.X = B[i].X;
			P.Y = B[j].Y; Q.Y = B[j].Y;
			P.Z = B[0].Z; Q.Z = B[1].Z;
			VertexData.Add((FVector)P);
			VertexData.Add((FVector)Q);

			P.Y = B[i].Y; Q.Y = B[i].Y;
			P.Z = B[j].Z; Q.Z = B[j].Z;
			P.X = B[0].X; Q.X = B[1].X;
			VertexData.Add((FVector)P);
			VertexData.Add((FVector)Q);

			P.Z = B[i].Z; Q.Z = B[i].Z;
			P.X = B[j].X; Q.X = B[j].X;
			P.Y = B[0].Y; Q.Y = B[1].Y;
			VertexData.Add((FVector)P);
			VertexData.Add((FVector)Q);
		}
	}

	UpdateElemBox();
}

void FKConvexElem::BakeTransformToVerts()
{
	for (int32 VertIdx = 0; VertIdx < VertexData.Num(); VertIdx++)
	{
		VertexData[VertIdx] = Transform.TransformPosition(VertexData[VertIdx]);
	}

	Transform = FTransform::Identity;
	UpdateElemBox();
}

FArchive& operator<<(FArchive& Ar,FKConvexElem& Elem)
{
	if (Ar.IsLoading())
	{
		// Initialize the TArray members
		FMemory::Memzero(&Elem.VertexData, sizeof(Elem.VertexData));
		FMemory::Memzero(&Elem.ElemBox, sizeof(Elem.ElemBox));
		Elem.ChaosConvex.SafeRelease();
	}

	return Ar;
}

bool FKLevelSetElem::Serialize(FArchive& Ar)
{
	Ar << Transform;

	if (Ar.IsLoading())
	{
		LevelSet = TSharedPtr<Chaos::FLevelSet>(new Chaos::FLevelSet());
	}

	if (LevelSet.IsValid())
	{
		LevelSet->Serialize(Ar);
	}

	return true;
}

bool FKSkinnedLevelSetElem::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		WeightedLatticeLevelSet = TRefCountPtr<Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>>(new Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>());
	}

	if (WeightedLatticeLevelSet.IsValid())
	{
		Chaos::FChaosArchive ChaosAr(Ar);
		WeightedLatticeLevelSet->Serialize(ChaosAr);
	}

	return true;
}

bool FKMLLevelSetElem::Serialize(FArchive& Ar)
{
	Ar << NNESignedDistanceModelData;
	Ar << NNEIncorrectZoneModelData;

	if (Ar.IsLoading())
	{
		MLLevelSet = TSharedPtr<Chaos::FMLLevelSet>(new Chaos::FMLLevelSet(NNESignedDistanceModelData, NNEIncorrectZoneModelData));
	}

	if (MLLevelSet.IsValid())
	{
		MLLevelSet->Serialize(Ar);
	}

	return true;
}

bool FKSkinnedTriangleMeshElem::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		SkinnedTriangleMesh = TRefCountPtr<Chaos::FSkinnedTriangleMesh>(new Chaos::FSkinnedTriangleMesh());
	}

	if (SkinnedTriangleMesh.IsValid())
	{
		Chaos::FChaosArchive ChaosAr(Ar);
		SkinnedTriangleMesh->Serialize(ChaosAr);
	}

	return true;
}
