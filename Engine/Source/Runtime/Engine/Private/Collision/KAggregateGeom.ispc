// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1
#define EXPLICIT_MATRIX 1

#include "Math/Vector.isph"
#include "Math/Matrix.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"

struct FRotator
{
	FReal Pitch;
	FReal Yaw;
	FReal Roll;
};

struct FBox
{
	FVector Min;
	FVector Max;
	unsigned int8 IsValid;
};

static inline uniform FVector4 RotatorToQuaternion(const uniform FRotator &InRotation)
{
	const uniform FVector4 Angles = SetVector4(InRotation.Pitch, InRotation.Yaw, InRotation.Roll, ZERO);
	const uniform FVector4 AnglesNoWinding = VectorMod(Angles, Vector360);
	const uniform FVector4 HalfAngles = VectorMultiply(AnglesNoWinding, DEG_TO_RAD_HALF);

	uniform FVector4 SinAngles, CosAngles;
	VectorSinCos(SinAngles, CosAngles, HalfAngles);

	// Vectorized conversion, measured 20% faster than using scalar version after VectorSinCos.
	// Indices within FVector4 (for shuffles): P=0, Y=1, R=2
	const uniform FVector4 SR = VectorReplicate(SinAngles, 2);
	const uniform FVector4 CR = VectorReplicate(CosAngles, 2);

	const uniform FVector4 SY_SY_CY_CY_Temp = VectorShuffle(SinAngles, CosAngles, 1, 1, 1, 1);

	const uniform FVector4 SP_SP_CP_CP = VectorShuffle(SinAngles, CosAngles, 0, 0, 0, 0);
	const uniform FVector4 SY_CY_SY_CY = VectorShuffle(SY_SY_CY_CY_Temp, SY_SY_CY_CY_Temp, 0, 2, 0, 2);

	const uniform FVector4 CP_CP_SP_SP = VectorShuffle(CosAngles, SinAngles, 0, 0, 0, 0);
	const uniform FVector4 CY_SY_CY_SY = VectorShuffle(SY_SY_CY_CY_Temp, SY_SY_CY_CY_Temp, 2, 0, 2, 0);

	const uniform FVector4 SignBitsLeft  = SetVector4(ZERO, REAL_BITS(SIGN_BIT), ZERO, ZERO); // {{ Pos, Neg, Pos, Pos }};
	const uniform FVector4 SignBitsRight = SetVector4(REAL_BITS(SIGN_BIT), REAL_BITS(SIGN_BIT), REAL_BITS(SIGN_BIT), ZERO); // {{ Neg, Neg, Neg, Pos }};
	const uniform FVector4 LeftTerm  = VectorBitwiseXor(SignBitsLeft , VectorMultiply(CR, VectorMultiply(SP_SP_CP_CP, SY_CY_SY_CY)));
	const uniform FVector4 RightTerm = VectorBitwiseXor(SignBitsRight, VectorMultiply(SR, VectorMultiply(CP_CP_SP_SP, CY_SY_CY_SY)));

	return VectorAdd(LeftTerm, RightTerm);
}

static inline void BoxTransformByMatrix(uniform FVector4 &BoxMin, uniform FVector4 &BoxMax, const uniform FMatrix &M)
{
	const uniform FVector4 VecMin = BoxMin;
	const uniform FVector4 VecMax = BoxMax;

	const uniform FVector4 Origin = VectorMultiply(VectorAdd(VecMax, VecMin), VectorOneHalf);
	const uniform FVector4 Extent = VectorMultiply(VectorSubtract(VecMax, VecMin), VectorOneHalf);

	const uniform FVector4 m0 = *((uniform FVector4 *uniform)&M.M[0]);
	const uniform FVector4 m1 = *((uniform FVector4 *uniform)&M.M[4]);
	const uniform FVector4 m2 = *((uniform FVector4 *uniform)&M.M[8]);
	const uniform FVector4 m3 = *((uniform FVector4 *uniform)&M.M[12]);

	uniform FVector4 NewOrigin = VectorMultiply(VectorReplicate(Origin, 0), m0);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(Origin, 1), m1, NewOrigin);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(Origin, 2), m2, NewOrigin);
	NewOrigin = VectorAdd(NewOrigin, m3);

	uniform FVector4 NewExtent = VectorAbs(VectorMultiply(VectorReplicate(Extent, 0), m0));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(Extent, 1), m1)));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(Extent, 2), m2)));

	BoxMin = VectorSubtract(NewOrigin, NewExtent);
	BoxMax = VectorAdd(NewOrigin, NewExtent);
}

static inline void BoxTransformByTransform(uniform FVector4 &BoxMin, uniform FVector4 &BoxMax, const uniform FTransform &T)
{
	const uniform FMatrix M = ToMatrixWithScale(T);
	BoxTransformByMatrix(BoxMin, BoxMax, M);
}

export void BoxCalcAABB(uniform FBox &LocalBox,
						const uniform FTransform &BoneTM,
						const uniform float Scale,
						const uniform FRotator &Rotation,
						const uniform FVector &Center,
						const uniform float X,
						const uniform float Y,
						const uniform float Z)
{
	uniform FTransform ElemTM;
	ElemTM.Rotation = RotatorToQuaternion(Rotation);
	ElemTM.Translation = SetVector4(Center.V[0], Center.V[1], Center.V[2], ZERO);
	ElemTM.Scale3D = Vector1110;

	ScaleTranslation(ElemTM, Scale);
	TransformMultiply(ElemTM, ElemTM, BoneTM);

	const uniform float HalfScale = 0.5f * Scale;
	const uniform FVector4f XYZ_0 = SetVector4(X, Y, Z, 0.0f);
	uniform FVector4 BoxMax = ConvertVector4fTo4Native(XYZ_0 * HalfScale);
	uniform FVector4 BoxMin = BoxMax * -ONE;

	BoxTransformByTransform(BoxMin, BoxMax, ElemTM);

	LocalBox.Min = SetVector(BoxMin.V[0], BoxMin.V[1], BoxMin.V[2]);
	LocalBox.Max = SetVector(BoxMax.V[0], BoxMax.V[1], BoxMax.V[2]);
	LocalBox.IsValid = 1;
}

export void SPhylCalcAABB(uniform FBox &Result,
						  const uniform FTransform &BoneTM,
						  const uniform float Scale,
						  const uniform FRotator &Rotation,
						  const uniform FVector &Center,
						  const uniform float Radius,
						  const uniform float Length)
{
	uniform FTransform ElemTM;
	ElemTM.Rotation = RotatorToQuaternion(Rotation);
	ElemTM.Translation = SetVector4(Center.V[0], Center.V[1], Center.V[2], ZERO);
	ElemTM.Scale3D = Vector1110;

	ScaleTranslation(ElemTM, Scale);
	TransformMultiply(ElemTM, ElemTM, BoneTM);

	// Get sphyl axis direction
	static const uniform FVector4 AxisZ = { {0.0f, 0.0f, 1.0f, 0.0f} };
	const uniform FVector4 Axis = TransformVector(ElemTM, AxisZ);
	// Get abs of that vector
	const uniform FVector4 AbsAxis = VectorAbs(Axis);
	// Scale by length of sphyl
	const uniform FVector4 AbsDist = AbsAxis * (Scale * 0.5f * Length);

	const uniform float Extent = Scale * Radius;
	const uniform FVector4 MinPos = (ElemTM.Translation - AbsDist) - Extent;
	const uniform FVector4 MaxPos = (ElemTM.Translation + AbsDist) + Extent;

	Result.Min = SetVector(MinPos.V[0], MinPos.V[1], MinPos.V[2]);
	Result.Max = SetVector(MaxPos.V[0], MaxPos.V[1], MaxPos.V[2]);
	Result.IsValid = 1;
}
