// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/PhysicsFiltering.h"
#include "CollisionQueryParams.h"

#include "Chaos/Vector32Int8.h"

void ExtractResponses(const FCollisionResponseContainer& InResponses, uint64& BlockingBits, uint64& TouchingBits)
{
#if !PLATFORM_ENABLE_VECTORINTRINSICS
	BlockingBits = 0;
	TouchingBits = 0;
	constexpr uint64 Count = UE_ARRAY_COUNT(InResponses.EnumArray);
	for (uint64 i = 0; i < Count; ++i)
	{
		const uint64 ChannelBit = (1LLU << i);
		if (InResponses.EnumArray[i] == ECR_Block)
		{
			BlockingBits |= ChannelBit;
		}
		else if (InResponses.EnumArray[i] == ECR_Overlap)
		{
			TouchingBits |= ChannelBit;
		}
	}
#else
	using namespace Chaos;
	BlockingBits = 0;
	TouchingBits = 0;
	static_assert(UE_ARRAY_COUNT(InResponses.EnumArray) == 64);

	const VectorRegister32Int8 OverlapRegister = Vector32Int8Set1(ECR_Overlap);
	const VectorRegister32Int8 BlockRegister = Vector32Int8Set1(ECR_Block);
	const int8* Data = (const int8*)(InResponses.EnumArray);

	const VectorRegister32Int8 LowChunk = VectorLoad32Int8(Data);
	const uint64 OverlapLow = VectorMaskBits(VectorCompareEQ(LowChunk, OverlapRegister));
	const uint64 BlockLow = VectorMaskBits(VectorCompareEQ(LowChunk, BlockRegister));

	const VectorRegister32Int8 HighChunk = VectorLoad32Int8(Data + 32);
	const uint64 OverlapHigh = VectorMaskBits(VectorCompareEQ(HighChunk, OverlapRegister));
	const uint64 BlockHigh = VectorMaskBits(VectorCompareEQ(HighChunk, BlockRegister));

	TouchingBits = (OverlapHigh << 32) | OverlapLow;
	BlockingBits = (BlockHigh << 32) | BlockLow;
#endif
}

//////////////////////////////////////////////////////////////////////////
// FPhysicsFilterBuilder

FPhysicsFilterBuilder::FPhysicsFilterBuilder(const FChaosScene* InScene)
{
}

FPhysicsFilterBuilder::FPhysicsFilterBuilder(const Chaos::Filter::FShapeFilterData& InShapeFilter, const Chaos::Filter::FInstanceData& InInstanceData)
	: InstanceData(InInstanceData)
	, ShapeFilterBuilder(InShapeFilter)
{
}

FPhysicsFilterBuilder::FPhysicsFilterBuilder(const FChaosScene* InScene, const Chaos::Filter::FShapeFilterData& InShapeFilter, const Chaos::Filter::FInstanceData& InInstanceData)
	: InstanceData(InInstanceData)
	, ShapeFilterBuilder(InShapeFilter)
{
}

FPhysicsFilterBuilder::FPhysicsFilterBuilder(TEnumAsByte<enum ECollisionChannel> InObjectType, FMaskFilter MaskFilter, const struct FCollisionResponseContainer& ResponseToChannels)
{
	SetCollisionChannelIndex(InObjectType);
	SetResponses(ResponseToChannels);
	SetMaskFilter(MaskFilter);
	ShapeFilterBuilder.SetFilterFlags(Chaos::EFilterFlags::None);
}

FPhysicsFilterBuilder& FPhysicsFilterBuilder::SetCollisionChannelIndex(const TEnumAsByte<enum ECollisionChannel> InObjectType)
{
	ShapeFilterBuilder.SetCollisionChannelIndex(InObjectType);
	return *this;
}

FPhysicsFilterBuilder& FPhysicsFilterBuilder::SetResponses(const FCollisionResponseContainer& InResponses)
{
	uint64 BlockingBits = 0;
	uint64 TouchingBits = 0;
	ExtractResponses(InResponses, BlockingBits, TouchingBits);
	ShapeFilterBuilder.SetBlockChannelMask(BlockingBits);
	ShapeFilterBuilder.SetOverlapChannelMask(TouchingBits);
	return *this;
}

FPhysicsFilterBuilder& FPhysicsFilterBuilder::SetMaskFilter(const FMaskFilter InMaskFilter)
{
	ShapeFilterBuilder.SetMaskFilter(InMaskFilter);
	return *this;
}

FPhysicsFilterBuilder& FPhysicsFilterBuilder::SetFlags(const Chaos::EFilterFlags InFlags, bool bEnabled)
{
	ShapeFilterBuilder.SetFilterFlags(InFlags, bEnabled);
	return *this;
}

FPhysicsFilterBuilder& FPhysicsFilterBuilder::SetOwnerID(const uint32 InOwnerId)
{
	InstanceData.SetOwnerId(InOwnerId);
	return *this;
}

FPhysicsFilterBuilder& FPhysicsFilterBuilder::SetComponentID(const uint32 InComponentID)
{
	InstanceData.SetComponentId(InComponentID);
	return *this;
}

Chaos::Filter::FShapeFilterData FPhysicsFilterBuilder::BuildShapeFilterData() const
{
	return ShapeFilterBuilder.Build();
}

Chaos::Filter::FInstanceData FPhysicsFilterBuilder::BuildInstanceData() const
{
	return InstanceData;
}

//////////////////////////////////////////////////////////////////////////
// FPhysicsObjectQueryFilterBuilder

FPhysicsObjectQueryFilterBuilder::FPhysicsObjectQueryFilterBuilder(const FChaosScene* InScene)
{
}

FPhysicsObjectQueryFilterBuilder& FPhysicsObjectQueryFilterBuilder::SetObjectTypes(const uint64 ObjectTypes)
{
	Builder.SetObjectTypes(ObjectTypes);
	return *this;
}

FPhysicsObjectQueryFilterBuilder& FPhysicsObjectQueryFilterBuilder::SetMultiQuery(const bool bMultiQuery)
{
	Builder.SetMultiQuery(bMultiQuery);
	return *this;
}

FPhysicsObjectQueryFilterBuilder& FPhysicsObjectQueryFilterBuilder::SetMaskFilter(const uint8 InMaskFilter)
{
	Builder.SetMaskFilter(InMaskFilter);
	return *this;
}

FPhysicsObjectQueryFilterBuilder& FPhysicsObjectQueryFilterBuilder::SetFlags(const Chaos::EFilterFlags InFlags, bool bEnabled)
{
	Builder.SetFilterFlags(InFlags, bEnabled);
	return *this;
}

Chaos::Filter::FQueryFilterData FPhysicsObjectQueryFilterBuilder::Build() const
{
	return Builder.Build();
}

//////////////////////////////////////////////////////////////////////////
// FPhysicsTraceQueryFilterBuilder

FPhysicsTraceQueryFilterBuilder::FPhysicsTraceQueryFilterBuilder(const FChaosScene* InScene)
{
}

FPhysicsTraceQueryFilterBuilder& FPhysicsTraceQueryFilterBuilder::SetCollisionChannelIndex(const TEnumAsByte<enum ECollisionChannel> InObjectType)
{
	Builder.SetCollisionChannelIndex(InObjectType);
	return *this;
}

FPhysicsTraceQueryFilterBuilder& FPhysicsTraceQueryFilterBuilder::SetResponses(const FCollisionResponseContainer& InResponses)
{
	uint64 BlockingMask = 0;
	uint64 OverlapMask = 0;
	ExtractResponses(InResponses, BlockingMask, OverlapMask);
	Builder.SetBlockChannelMask(BlockingMask);
	Builder.SetOverlapChannelMask(OverlapMask);
	return *this;
}

FPhysicsTraceQueryFilterBuilder& FPhysicsTraceQueryFilterBuilder::SetMaskFilter(const FMaskFilter InMaskFilter)
{
	Builder.SetMaskFilter(InMaskFilter);
	return *this;
}

FPhysicsTraceQueryFilterBuilder& FPhysicsTraceQueryFilterBuilder::SetFlags(const Chaos::EFilterFlags InFlags, bool bEnabled)
{
	Builder.SetFilterFlags(InFlags, bEnabled);
	return *this;
}

Chaos::Filter::FQueryFilterData FPhysicsTraceQueryFilterBuilder::Build() const
{
	return Builder.Build();
}

ECollisionChannel GetCollisionChannel(const Chaos::Filter::FShapeFilterData& ShapeFilterData)
{
	return (ECollisionChannel)ShapeFilterData.GetCollisionChannelIndex();
}

ECollisionChannel GetCollisionChannel(const Chaos::Filter::FQueryFilterData& QueryFilterData)
{
	return (ECollisionChannel)QueryFilterData.GetCollisionChannelIndex();
}

bool HasCollisionChannel(const Chaos::Filter::FShapeFilterData& ShapeFilterData, ECollisionChannel Channel)
{
	return ShapeFilterData.IsCollisionChannelSet(Channel);
}

bool HasCollisionChannel(const Chaos::Filter::FQueryFilterData& QueryFilterData, ECollisionChannel Channel)
{
	return QueryFilterData.IsCollisionChannelSet(Channel);
}

FCollisionResponseContainer ExtractCollisionResponseContainer(const Chaos::Filter::FShapeFilterData& ShapeFilterData)
{
	FCollisionResponseContainer CollisionResponseContainer;
	const uint64 BlockChannels = ShapeFilterData.GetBlockChannels();
	const uint64 OverlapChannels = ShapeFilterData.GetOverlapChannels();

	for (int32 ChannelIndex = 0; ChannelIndex < UE_ARRAY_COUNT(CollisionResponseContainer.EnumArray); ++ChannelIndex)
	{
		const uint64 ChannelMask = ((uint64)1 << ChannelIndex);
		if (BlockChannels & ChannelMask)
		{
			CollisionResponseContainer.EnumArray[ChannelIndex] = ECR_Block;
		}
		else if (OverlapChannels & ChannelMask)
		{
			CollisionResponseContainer.EnumArray[ChannelIndex] = ECR_Overlap;
		}
		else
		{
			CollisionResponseContainer.EnumArray[ChannelIndex] = ECR_Ignore;
		}
	}

	return CollisionResponseContainer;
}
