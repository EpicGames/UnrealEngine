// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/World.h"
#include "Engine/OverlapResult.h"
#include "CollisionDebugDrawingPublic.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/PhysicsInterfaceUtils.h"
#include "Physics/PhysicsQueryHandler.h"
#include "Physics/SceneQueryData.h"
#include "Collision/CollisionConversions.h"
#include "PhysicsEngine/ClusterUnionComponent.h"
#include "PhysicsEngine/CollisionQueryFilterCallback.h"
#include "PhysicsEngine/ExternalSpatialAccelerationPayload.h"
#include "PhysicsEngine/ScopedSQHitchRepeater.h"
#include "Chaos/ChaosArchive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#include "Collision/CollisionDebugDrawing.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#endif

#include "ChaosVDSQTraceHelper.h"

float DebugLineLifetime = 2.f;

#include "PhysicsEngine/CollisionAnalyzerCapture.h"

#include "Physics/Experimental/ChaosInterfaceWrapper.h"
#include "PBDRigidsSolver.h"

CSV_DEFINE_CATEGORY(SceneQuery, false);

#if UE_WITH_REMOTE_OBJECT_HANDLE
thread_local bool bIgnoreQueryHandler = false;
#endif

bool GetIgnoreQueryHandler()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
return bIgnoreQueryHandler;
#else
return false;
#endif
}

enum class ESingleMultiOrTest : uint8
{
	Single,
	Multi,
	Test
};

enum class ESweepOrRay : uint8
{
	Raycast,
	Sweep,
};

struct FGeomSQAdditionalInputs
{
	FGeomSQAdditionalInputs(const FCollisionShape& InCollisionShape, const FQuat& InGeomRot)
		: ShapeAdapter(InGeomRot, InCollisionShape)
		, CollisionShape(InCollisionShape)
	{
	}

	const FPhysicsGeometry* GetGeometry() const
	{
		return &ShapeAdapter.GetGeometry();
	}

	const FQuat* GetGeometryOrientation() const
	{
		return &ShapeAdapter.GetGeomOrientation();
	}


	const FCollisionShape* GetCollisionShape() const
	{
		return &CollisionShape;
	}

	FPhysicsShapeAdapter ShapeAdapter;
	const FCollisionShape& CollisionShape;
};

struct FGeomCollectionSQAdditionalInputs
{
	FGeomCollectionSQAdditionalInputs(const FPhysicsGeometryCollection& InCollection, const FQuat& InGeomRot)
	: Collection(InCollection)
	, GeomRot(InGeomRot)
	{
	}

	const FPhysicsGeometry* GetGeometry() const
	{
		return &Collection.GetGeometry();
	}

	const FQuat* GetGeometryOrientation() const
	{
		return &GeomRot;
	}

	const FPhysicsGeometryCollection* GetCollisionShape() const
	{
		return &Collection;
	}

	const FPhysicsGeometryCollection& Collection;
	const FQuat& GeomRot;
};

struct FPhysicsGeometrySQAdditionalInputs
{
	FPhysicsGeometrySQAdditionalInputs(const FPhysicsGeometry& InGeometry, const FQuat& InGeomRot)
		: Collection(FChaosEngineInterface::GetGeometryCollection(InGeometry))
		, GeomRot(InGeomRot)
	{
	}

	const FPhysicsGeometry* GetGeometry() const
	{
		return &Collection.GetGeometry();
	}

	const FQuat* GetGeometryOrientation() const
	{
		return &GeomRot;
	}

	const FPhysicsGeometryCollection* GetCollisionShape() const
	{
		return &Collection;
	}
private:
	const FPhysicsGeometryCollection Collection;
	const FQuat& GeomRot;
};

struct FRaycastSQAdditionalInputs
{
	const FPhysicsGeometry* GetGeometry() const
	{
		return nullptr;
	}

	const FQuat* GetGeometryOrientation() const
	{
		return nullptr;
	}

	const FCollisionShape* GetCollisionShape() const
	{
		return nullptr;
	}
};


template<typename GeomType>
using TGeomSQInputs = std::conditional_t<std::is_same_v<GeomType, FPhysicsGeometryCollection>, FGeomCollectionSQAdditionalInputs,
	std::conditional_t<std::is_same_v<GeomType, FCollisionShape>, FGeomSQAdditionalInputs,
	std::conditional_t<std::is_same_v<GeomType, FPhysicsGeometry>, FPhysicsGeometrySQAdditionalInputs,
	void>>>;

template<typename GeomType>
TGeomSQInputs<GeomType> GeomToSQInputs(const GeomType& Geom, const FQuat& Rot)
{
	static_assert(!std::is_same_v<TGeomSQInputs<GeomType>, void>, "Invalid geometry passed to SQ.");
	if constexpr (std::is_same_v<GeomType, FPhysicsGeometryCollection>)
	{
		return FGeomCollectionSQAdditionalInputs(Geom, Rot);
	}
	else if constexpr (std::is_same_v<GeomType, FPhysicsGeometry>)
	{
		return FPhysicsGeometrySQAdditionalInputs(Geom, Rot);
	}
	else if constexpr (std::is_same_v<GeomType, FCollisionShape>)
	{
		return FGeomSQAdditionalInputs(Geom, Rot);
	}
}

bool BuildQueryShape(const FCollisionShape& CollisionShape, Chaos::FQueryShape& OutQueryShape)
{
	OutQueryShape.CollisionShape = CollisionShape;
	return true;
}

bool BuildQueryShape(const FPhysicsGeometryCollection& GeometryCollection, Chaos::FQueryShape& OutQueryShape)
{
	ECollisionShapeType ShapeType = GeometryCollection.GetType();
	switch (ShapeType)
	{
		case ECollisionShapeType::Box:
		{
			const Chaos::TBox<Chaos::FReal, 3>& Box = GeometryCollection.GetBoxGeometry();
			OutQueryShape.CollisionShape = FCollisionShape::MakeBox(Box.Extents() * 0.5f);
			return true;
		}
		case ECollisionShapeType::Sphere:
		{
			const Chaos::FSphere& Sphere = GeometryCollection.GetSphereGeometry();
			OutQueryShape.CollisionShape = FCollisionShape::MakeSphere(Sphere.GetRadiusf());
			return true;
		}
		case ECollisionShapeType::Capsule:
		{
			const Chaos::FCapsule& Capsule = GeometryCollection.GetCapsuleGeometry();
			const float Radius = Capsule.GetRadiusf();
			const float HalfHeight = Capsule.GetHeightf() * 0.5f;
			// Note: FCollisionShape and FCapsule disagree on what the (half) height are. 
			// FCapsule defines height as the distance between the sphere centers while FCollision defines it as the distance between the full top to bottom.
			const float FullHalfHeight = HalfHeight + Radius;
			OutQueryShape.CollisionShape = FCollisionShape::MakeCapsule(Radius, FullHalfHeight);
			return true;
		}
		case ECollisionShapeType::Convex:
		{
			FMemoryWriter MemoryWriter(OutQueryShape.ConvexData);
			Chaos::FChaosArchive Writer(MemoryWriter);

			const Chaos::FImplicitObject& Geometry = GeometryCollection.GetGeometry();
			Chaos::FImplicitObjectPtr CopiedGeometryPtr = Geometry.CopyGeometry();
			Chaos::FImplicitObject& CopiedGeometry = *CopiedGeometryPtr;

			Chaos::FImplicitObject::SerializationFactory(Writer, &CopiedGeometry);
			CopiedGeometry.Serialize(Writer);
			OutQueryShape.LocalBoundingBox = CopiedGeometry.BoundingBox();
			return true;
		}
		default:
		{
			UE_LOGF(LogChaos, Warning, "PhysicsQueryHandler: Invalid shape type. Shape type %d is not yet supported", (int)ShapeType);
		}
	}
	return false;
}

void ConvertToConvex(const Chaos::FQueryShape& QueryShape, Chaos::FImplicitObjectPtr& OutConvex)
{
	ensure(QueryShape.IsConvexShape());

	FMemoryReader MemoryReader(QueryShape.ConvexData);
	Chaos::FChaosArchive Reader(MemoryReader);

	OutConvex = Chaos::FImplicitObject::SerializationFactory(Reader, nullptr);
	OutConvex->Serialize(Reader);
}

template <typename InHitType, ESweepOrRay InGeometryQuery, ESingleMultiOrTest InSingleMultiOrTest>
struct TSQTraits
{
	static const ESingleMultiOrTest SingleMultiOrTest = InSingleMultiOrTest;
	static const ESweepOrRay GeometryQuery = InGeometryQuery;
	using HitType = InHitType;
	using OutHitsType = std::conditional_t<InSingleMultiOrTest == ESingleMultiOrTest::Multi, TArray<FHitResult>, FHitResult>;
	using HitBufferType = std::conditional_t<InSingleMultiOrTest == ESingleMultiOrTest::Multi, FDynamicHitBuffer<InHitType>, FSingleHitBuffer<InHitType>>;

	// GetNumHits - multi
	template <ESingleMultiOrTest T = SingleMultiOrTest>
	static typename TEnableIf<T == ESingleMultiOrTest::Multi, int32>::Type GetNumHits(const HitBufferType& HitBuffer)
	{
		return HitBuffer.GetNumHits();
	}

	// GetNumHits - single/test
	template <ESingleMultiOrTest T = SingleMultiOrTest>
	static typename TEnableIf<T != ESingleMultiOrTest::Multi, int32>::Type GetNumHits(const HitBufferType& HitBuffer)
	{
		return GetHasBlock(HitBuffer) ? 1 : 0;
	}

	//GetHits - multi
	template <ESingleMultiOrTest T = SingleMultiOrTest>
	static typename TEnableIf<T == ESingleMultiOrTest::Multi, HitType*>::Type GetHits(HitBufferType& HitBuffer)
	{
		return HitBuffer.GetHits();
	}

	//GetHits - single/test
	template <ESingleMultiOrTest T = SingleMultiOrTest>
	static typename TEnableIf<T != ESingleMultiOrTest::Multi, HitType*>::Type GetHits(HitBufferType& HitBuffer)
	{
		return GetBlock(HitBuffer);
	}

	//SceneTrace - ray
	template <typename AccelContainerType, typename GeomInputsType, ESweepOrRay T = GeometryQuery>
	static typename TEnableIf<T == ESweepOrRay::Raycast, void>::Type SceneTrace(const AccelContainerType& Container, const GeomInputsType& GeomInputs, const FVector& Dir, float DeltaMag, const FTransform& StartTM, HitBufferType& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams)
	{
		Chaos::Private::LowLevelRaycast(Container, StartTM.GetLocation(), Dir, DeltaMag, HitBuffer, OutputFlags, CommonParams);
	}

	//SceneTrace - sweep
	template <typename AccelContainerType, typename GeomInputsType, ESweepOrRay T = GeometryQuery>
	static typename TEnableIf<T == ESweepOrRay::Sweep, void>::Type SceneTrace(const AccelContainerType& Container, const GeomInputsType& GeomInputs, const FVector& Dir, float DeltaMag, const FTransform& StartTM, HitBufferType& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams)
	{
		Chaos::Private::LowLevelSweep(Container, *GeomInputs.GetGeometry(), StartTM, Dir, DeltaMag, HitBuffer, OutputFlags, CommonParams);
	}

	static void ResetOutHits(TArray<FHitResult>& OutHits, const FVector& Start, const FVector& End)
	{
		OutHits.Reset();
	}

	static void ResetOutHits(FHitResult& OutHit, const FVector& Start, const FVector& End)
	{
		OutHit = FHitResult();
		OutHit.TraceStart = Start;
		OutHit.TraceEnd = End;
	}

	static void DrawTraces(const UWorld* World, const FVector& Start, const FVector& End, const FPhysicsGeometry* PGeom, const FQuat* PGeomRot, const TArray<FHitResult>& Hits)
	{
		if (IsRay())
		{
			DrawLineTraces(World, Start, End, Hits, DebugLineLifetime);
		}
		else
		{
			DrawGeomSweeps(World, Start, End, *PGeom, *PGeomRot, Hits, DebugLineLifetime);
		}
	}

	static void DrawTraces(const UWorld* World, const FVector& Start, const FVector& End, const FPhysicsGeometry* PGeom, const FQuat* GeomRotation, const FHitResult& Hit)
	{
		TArray<FHitResult> Hits;
		Hits.Add(Hit);

		DrawTraces(World, Start, End, PGeom, GeomRotation, Hits);
	}

	template <typename GeomInputsType>
	static void CaptureTraces(const UWorld* World, const FVector& Start, const FVector& End, const GeomInputsType& GeomInputs, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams, const TArray<FHitResult>& Hits, bool bHaveBlockingHit, double StartTime)
	{
#if ENABLE_COLLISION_ANALYZER
		ECAQueryMode::Type QueryMode = IsMulti() ? ECAQueryMode::Multi : (IsSingle() ? ECAQueryMode::Single : ECAQueryMode::Test);
		if (IsRay())
		{
			CAPTURERAYCAST(World, Start, End, QueryMode, TraceChannel, Params, ResponseParams, ObjectParams, Hits);
		}
		else
		{
			CAPTUREGEOMSWEEP(World, Start, End, *GeomInputs.GetGeometryOrientation(), QueryMode, *GeomInputs.GetCollisionShape(), TraceChannel, Params, ResponseParams, ObjectParams, Hits);
		}
#endif
	}

	template <typename GeomInputsType>
	static void CaptureTraces(const UWorld* World, const FVector& Start, const FVector& End, const GeomInputsType& GeomInputs, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams, const FHitResult& Hit, bool bHaveBlockingHit, double StartTime)
	{
		TArray<FHitResult> Hits;
		if (bHaveBlockingHit)
		{
			Hits.Add(Hit);
		}
		CaptureTraces(World, Start, End, GeomInputs, TraceChannel, Params, ResponseParams, ObjectParams, Hits, bHaveBlockingHit, StartTime);
	}

	static EHitFlags GetHitFlags()
	{
		if (IsTest())
		{
			return EHitFlags::None;
		}
		else
		{
			if (IsRay())
			{
				return EHitFlags::Position | EHitFlags::Normal | EHitFlags::Distance | EHitFlags::MTD | EHitFlags::FaceIndex;
			}
			else
			{
				if (IsSingle())
				{
					return EHitFlags::Position | EHitFlags::Normal | EHitFlags::Distance | EHitFlags::MTD;
				}
				else
				{
					return EHitFlags::Position | EHitFlags::Normal | EHitFlags::Distance | EHitFlags::MTD | EHitFlags::FaceIndex;
				}
			}
		}
	}

	static EQueryFlags GetQueryFlags()
	{
		if (IsRay())
		{
			return (IsTest() ? (EQueryFlags::PreFilter | EQueryFlags::AnyHit) : EQueryFlags::PreFilter);
		}
		else
		{
			if (IsTest())
			{
				return (EQueryFlags::PreFilter | EQueryFlags::PostFilter | EQueryFlags::AnyHit);
			}
			else if (IsSingle())
			{
				return EQueryFlags::PreFilter;
			}
			else
			{
				return (EQueryFlags::PreFilter | EQueryFlags::PostFilter);
			}
		}
	}

	CA_SUPPRESS(6326);
	constexpr static bool IsSingle() { return SingleMultiOrTest == ESingleMultiOrTest::Single;  }

	CA_SUPPRESS(6326);
	constexpr static bool IsTest() { return SingleMultiOrTest == ESingleMultiOrTest::Test;  }

	CA_SUPPRESS(6326);
	constexpr static bool IsMulti() { return SingleMultiOrTest == ESingleMultiOrTest::Multi;  }

	CA_SUPPRESS(6326);
	constexpr static bool IsRay() { return GeometryQuery == ESweepOrRay::Raycast;  }

	CA_SUPPRESS(6326);
	constexpr static bool IsSweep() { return GeometryQuery == ESweepOrRay::Sweep;  }

	/** Easy way to query whether this SQ trait is for the GT or the PT based on the based in hit type. */
	constexpr static bool IsExternalData() { return std::is_base_of_v<ChaosInterface::FActorShape, InHitType>; }

};

using EThreadQueryContext = Chaos::EThreadQueryContext;

EThreadQueryContext GetThreadQueryContext(const Chaos::FPhysicsSolver& Solver)
{
	if (Solver.IsGameThreadFrozen())
	{
		//If the game thread is frozen the solver is currently in fixed tick mode (i.e. fixed tick callbacks are being executed on GT)
		if (IsInGameThread() || IsInParallelGameThread())
		{
			//Since we are on GT or parallel GT we must be in fixed tick, so use PT data and convert back to GT where possible
			return EThreadQueryContext::PTDataWithGTObjects;
		}
		else
		{
			//The solver can't be running since it's calling fixed tick callbacks on gt, so it must be an unrelated thread task (audio, animation, etc...) so just use interpolated gt data
			return EThreadQueryContext::GTData;
		}
	}
	else
	{
		//TODO: need a way to know we are on a physics thread task (this isn't supported yet)
		//For now just use interpolated data
		return EThreadQueryContext::GTData;
	}
}

struct FClusterUnionHit
{
	bool bIsClusterUnion = false;
	bool bHit = false;
};

struct FDefaultAccelContainer
{
	static constexpr bool HasAccelerationStructureOverride()
	{
		return false;
	}
};

template<typename AccelType>
struct FOverrideAccelContainer
{
	explicit FOverrideAccelContainer(const AccelType& InSpatialAcceleration)
		: SpatialAcceleration(InSpatialAcceleration)
	{}

	static constexpr bool HasAccelerationStructureOverride()
	{
		return true;
	}

	const AccelType& GetSpatialAcceleration() const
	{
		return SpatialAcceleration;
	}

private:
	const AccelType& SpatialAcceleration;
};

template <typename Traits, typename GeomInputsType, typename AccelContainerType>
bool TSceneCastCommonImpWithRetryRequest(const UWorld* World, typename Traits::OutHitsType& OutHits, const GeomInputsType& GeomInputs, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const AccelContainerType& AccelContainer, bool& bOutRequestRetry, FCollisionQueryParams& OutRetryParams)
{
	using namespace ChaosInterface;
	bOutRequestRetry = false;

	FScopeCycleCounter Counter(Params.StatId);
	STARTQUERYTIMER();

	if (!Traits::IsTest())
	{
		Traits::ResetOutHits(OutHits, Start, End);
	}

	// Track if we get any 'blocking' hits
	bool bHaveBlockingHit = false;

	FVector Delta = End - Start;
	float DeltaSize = Delta.Size();
	float DeltaMag = FMath::IsNearlyZero(DeltaSize) ? 0.f : DeltaSize;
	float MinBlockingDistance = DeltaMag;
	if (Traits::IsSweep() || DeltaMag > 0.f)
	{
		// Create filter data used to filter collisions 
		CA_SUPPRESS(6326);
		const Chaos::Filter::FQueryFilterData QueryFilterData = CreateChaosQueryFilterData(TraceChannel, Params.bTraceComplex, ResponseParams.CollisionResponse, Params, ObjectParams, Traits::SingleMultiOrTest == ESingleMultiOrTest::Multi);
		
		CA_SUPPRESS(6326);
		FCollisionQueryFilterCallback QueryCallback(Params, Traits::GeometryQuery == ESweepOrRay::Sweep);

		CA_SUPPRESS(6326);
		if (Traits::SingleMultiOrTest != ESingleMultiOrTest::Multi)
		{
			QueryCallback.bIgnoreTouches = true;
		}

		typename Traits::HitBufferType HitBufferSync;

		bool bBlockingHit = false;
		const FVector Dir = DeltaMag > 0.f ? (Delta / DeltaMag) : FVector(1, 0, 0);
		const FTransform StartTM = Traits::IsRay() ? FTransform(Start) : FTransform(*GeomInputs.GetGeometryOrientation(), Start);

		FQueryDebugParams DebugParams;
#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
		DebugParams.bDebugQuery = Params.bDebugQuery;
#endif

		// Enable scene locks, in case they are required
		FPhysScene& PhysScene = *World->GetPhysicsScene();
		
		FScopedSceneReadLock SceneLocks(PhysScene);
		{
			const ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback, QueryFilterData, Traits::GetQueryFlags(), DebugParams);
			FScopedSQHitchRepeater<decltype(HitBufferSync)> HitchRepeater(HitBufferSync, QueryCallback, FHitchDetectionInfo(Start, End, TraceChannel, Params));
			do
			{
				if constexpr (AccelContainerType::HasAccelerationStructureOverride())
				{
					Traits::SceneTrace(AccelContainer.GetSpatialAcceleration(), GeomInputs, Dir, DeltaMag, StartTM, HitchRepeater.GetBuffer(), Traits::GetHitFlags(), CommonParams);
				}
				else
				{
					Traits::SceneTrace(PhysScene, GeomInputs, Dir, DeltaMag, StartTM, HitchRepeater.GetBuffer(), Traits::GetHitFlags(), CommonParams);
				}
			} while (HitchRepeater.RepeatOnHitch());
		}


		const int32 NumHits = Traits::GetNumHits(HitBufferSync);

		if(NumHits > 0 && GetHasBlock(HitBufferSync))
		{
			bBlockingHit = true;
			MinBlockingDistance = GetDistance(Traits::GetHits(HitBufferSync)[NumHits - 1]);
		}
		if (NumHits > 0 && !Traits::IsTest())
		{
			bool bSuccess = ConvertTraceResults(bBlockingHit, World, NumHits, Traits::GetHits(HitBufferSync), DeltaMag, QueryFilterData, OutHits, Start, End, GeomInputs.GetGeometry(), StartTM, MinBlockingDistance, Params.bReturnFaceIndex, Params.bReturnPhysicalMaterial) == EConvertQueryResult::Valid;

			if (!bSuccess)
			{
				// We don't need to change bBlockingHit, that's done by ConvertTraceResults if it removed the blocking hit.
				UE_LOGF(LogCollision, Error, "%ls%ls resulted in a NaN/INF in PHit!", Traits::IsRay() ? TEXT("Raycast") : TEXT("Sweep"), Traits::IsMulti() ? TEXT("Multi") : (Traits::IsSingle() ? TEXT("Single") : TEXT("Test")));
#if ENABLE_NAN_DIAGNOSTIC
				UE_LOGF(LogCollision, Error, "--------TraceChannel : %d", (int32)TraceChannel);
				UE_LOGF(LogCollision, Error, "--------Start : %ls", *Start.ToString());
				UE_LOGF(LogCollision, Error, "--------End : %ls", *End.ToString());
				if (Traits::IsSweep())
				{
					UE_LOGF(LogCollision, Error, "--------GeomRotation : %ls", *(GeomInputs.GetGeometryOrientation()->ToString()));
				}
				UE_LOGF(LogCollision, Error, "--------%ls", *Params.ToString());
#endif
			}

			// This block is only necessary on the game thread when dealing with cluster unions.
			// TODO: Is there a way to get this to generalize better to the PT as well? Right now it depends on GT functions on the cluster union component.
			if constexpr (Traits::IsExternalData())
			{
				auto DoClusterUnionTrace = [&GeomInputs, &Start, &End, TraceChannel, &Params, &ResponseParams, &ObjectParams](const FHitResult& OriginalHit, auto& NewHit)
				{
					FClusterUnionHit Result;

#if !UE_WITH_REMOTE_OBJECT_HANDLE
					if (UClusterUnionComponent* ClusterUnion = Cast<UClusterUnionComponent>(OriginalHit.GetComponent()))
					{
						Result.bIsClusterUnion = true;
						if constexpr (Traits::IsRay())
						{
							Result.bHit = ClusterUnion->LineTraceComponent(NewHit, Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
						}
						else
						{
							Result.bHit = ClusterUnion->SweepComponent(NewHit, Start, End, *GeomInputs.GetGeometryOrientation(), *GeomInputs.GetGeometry(), TraceChannel, Params, ResponseParams, ObjectParams);
						}
					}
#endif

					return Result;
				};
				if (bSuccess && Params.bTraceIntoSubComponents)
				{
					if constexpr (Traits::IsMulti())
					{
						const bool bHadBlockingHit = bBlockingHit;
						TArray<FHitResult> AllNewHits;
						TArray<int32> ClusterUnionIndices;
						TArray<AActor*, TInlineAllocator<1>> ClusterUnionActorsToIgnoreIfRetry;
						bBlockingHit = false;
						bool bResortHits = false;

						for (int32 Index = 0; Index < OutHits.Num(); ++Index)
						{
							TArray<FHitResult> NewHit;
							FClusterUnionHit ClusterUnionHit = DoClusterUnionTrace(OutHits[Index], NewHit);
							if (ClusterUnionHit.bIsClusterUnion)
							{
								if (Params.bReplaceHitWithSubComponents || !ClusterUnionHit.bHit)
								{
									ClusterUnionIndices.Add(Index);
								}

								if (ClusterUnionHit.bHit)
								{
									bBlockingHit = true;
									AllNewHits.Append(NewHit);
								}
								else if (OutHits[Index].bBlockingHit)
								{
									// Subtrace has no blocking hit but the cluster union trace was a blocking hit.
									// We need to make sure this cluster union gets ignored if we retry.
									ClusterUnionActorsToIgnoreIfRetry.Add(OutHits[Index].GetActor());
								}
							}
							else
							{
								bBlockingHit |= OutHits[Index].bBlockingHit;
							}
						}

						if (bHadBlockingHit && !bBlockingHit)
						{
							// We had a blocking hit, but after subtracing against a cluster union we no longer have a blocking hit because its subcomponent(s) were ignored (e.g. if ignored actors/ignored components is used).
							// In this case we want to retry and ignore the cluster unions that were hit to continue the trace until it reaches the end/finds another blocking hit.
							Traits::ResetOutHits(OutHits, Start, End);
							OutRetryParams = Params;
							for (AActor* IgnoreActor : ClusterUnionActorsToIgnoreIfRetry)
							{
								OutRetryParams.AddIgnoredActor(IgnoreActor);
							}
							bOutRequestRetry = true;
							return false;
						}
						else
						{
							for (int32 Index = ClusterUnionIndices.Num() - 1; Index >= 0; --Index)
							{
								// No shrinking since we're going to be adding more elements shortly.
								OutHits.RemoveAtSwap(ClusterUnionIndices[Index], EAllowShrinking::No);
								bResortHits = true;
							}
	
							if (Params.bReplaceHitWithSubComponents)
							{
								OutHits.Append(AllNewHits);
								bResortHits = true;
							}
						}

						if (bResortHits)
						{
							OutHits.Sort(FCompareFHitResultTime());
						}
					}
					else
					{
						FHitResult NewHit;
						FClusterUnionHit ClusterUnionHit = DoClusterUnionTrace(OutHits, NewHit);
						if (ClusterUnionHit.bIsClusterUnion)
						{
							bBlockingHit = ClusterUnionHit.bHit;
							if (ClusterUnionHit.bHit)
							{
								if (Params.bReplaceHitWithSubComponents)
								{
									OutHits = NewHit;
								}
							}
							else if (AActor* ClusterUnionActor = OutHits.GetActor())
							{
								// In the case where the trace hits a cluster union but after we subtrace the cluster union and we end up finding out we hit *nothing* then we actually need to
								// redo the entire trace and force the SQ trace to ignore the cluster union actor. This is only relevant in the case where we aren't doing a multi-trace since in the
								// case of a multi-trace, we would've found all the other things we hit as well. In the case of a non-multi (single) trace, the SQ trace determined that the cluster union
								// is the best hit! But there could be other things we could've hit instead if we ignored the fact that we hit a cluster union.
								OutRetryParams = Params;
								// Ignore the actor that was hit (Check the shape data to be sure)
								const Chaos::Filter::FInstanceData& FilterInstanceData = Traits::GetHits(HitBufferSync)->Shape->GetFilterInstanceData();
								uint32 ActorIDFromShape = FilterInstanceData.GetOwnerId();
								OutRetryParams.AddIgnoredActor(ActorIDFromShape);
								if (ActorIDFromShape != ClusterUnionActor->GetUniqueID())
								{
									UE_LOGF(LogChaos, Warning, "TSceneCastCommonImpWithRetryRequest: Incorrect Shape Actor ID detected");
								}								
								bOutRequestRetry = true;
								return false;
							}
							else
							{
								Traits::ResetOutHits(OutHits, Start, End);
							}
						}
					}
				}
			}
		}

		bHaveBlockingHit = bBlockingHit;

	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (World->DebugDrawSceneQueries(Params.TraceTag))
	{
		Traits::DrawTraces(World, Start, End, GeomInputs.GetGeometry(), GeomInputs.GetGeometryOrientation(), OutHits);
	}
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#if ENABLE_COLLISION_ANALYZER
	Traits::CaptureTraces(World, Start, End, GeomInputs, TraceChannel, Params, ResponseParams, ObjectParams, OutHits, bHaveBlockingHit, StartTime);
#endif

	return bHaveBlockingHit;
}

template <typename Traits, typename GeomInputsType, typename AccelContainerType>
bool TSceneCastCommonImp(const UWorld* World, typename Traits::OutHitsType& OutHits, const GeomInputsType& GeomInputs, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const AccelContainerType& AccelContainer)
{
	bool bRequestRetry = true;
	bool bReturnResult = false;
	FCollisionQueryParams RetryParams;

	{
		constexpr bool bIsRetryQuery = false;
		CVD_TRACE_SCOPED_SCENE_QUERY_HELPER(World, GeomInputs.GetGeometry(), FTransform(GeomInputs.GetGeometryOrientation() ? *GeomInputs.GetGeometryOrientation() : FQuat::Identity, Start), End, TraceChannel, Params, ResponseParams, ObjectParams, Traits::IsSweep() ? EChaosVDSceneQueryType::Sweep : EChaosVDSceneQueryType::RayCast, static_cast<EChaosVDSceneQueryMode>(Traits::SingleMultiOrTest), bIsRetryQuery);
		bReturnResult = TSceneCastCommonImpWithRetryRequest<Traits, GeomInputsType, AccelContainerType>(World, OutHits, GeomInputs, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer, bRequestRetry, RetryParams);
	}

	int InfiniteLoopProtection = 10;
	while (bRequestRetry && InfiniteLoopProtection > 0)
	{
		CVD_TRACE_SCOPED_SCENE_QUERY_HELPER(World, GeomInputs.GetGeometry(), FTransform(GeomInputs.GetGeometryOrientation() ? *GeomInputs.GetGeometryOrientation() : FQuat::Identity, Start), End, TraceChannel, Params, ResponseParams, ObjectParams, Traits::IsSweep() ? EChaosVDSceneQueryType::Sweep : EChaosVDSceneQueryType::RayCast, static_cast<EChaosVDSceneQueryMode>(Traits::SingleMultiOrTest), bRequestRetry);
		bReturnResult = TSceneCastCommonImpWithRetryRequest<Traits, GeomInputsType, AccelContainerType>(World, OutHits, GeomInputs, Start, End, TraceChannel, RetryParams, ResponseParams, ObjectParams, AccelContainer, bRequestRetry, RetryParams);
		InfiniteLoopProtection--;
	}

	if (InfiniteLoopProtection <= 0)
	{
		UE_LOGF(LogChaos, Warning, "TSceneCastCommonImp: Potential Infinite Loop Detected");
		bReturnResult = false;
	}

	return bReturnResult;
}

// This selects the GT/PT traits based upon the given thread context.
template <typename HitType, typename PTHitType, ESweepOrRay SweepOrRay, ESingleMultiOrTest SingleMultiOrTest, typename GeomInputsType, typename OutHitsType, typename AccelContainerType>
bool TraceCommonImp(const EThreadQueryContext ThreadContext, const UWorld* World, OutHitsType& OutHits, const GeomInputsType& GeomInputs, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const AccelContainerType& AccelContainer)
{
	using namespace ChaosInterface;
	using Traits = TSQTraits<HitType, SweepOrRay, SingleMultiOrTest>;
	using PTTraits = TSQTraits<PTHitType, SweepOrRay, SingleMultiOrTest>;
	if (ThreadContext == EThreadQueryContext::GTData)
	{
		return TSceneCastCommonImp<Traits, GeomInputsType>(World, OutHits, GeomInputs, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
	}
	else
	{
		return TSceneCastCommonImp<PTTraits, GeomInputsType>(World, OutHits, GeomInputs, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
	}
}

// This converts ESweepOrRay to the two different GT/PT Hit types
template <ESweepOrRay SweepOrRay, ESingleMultiOrTest SingleMultiOrTest, typename GeomInputsType, typename OutHitsType, typename AccelContainerType>
bool TraceCommonImp(const EThreadQueryContext ThreadContext, const UWorld* World, OutHitsType& OutHits, const GeomInputsType& GeomInputs, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const AccelContainerType& AccelContainer)
{
	using namespace ChaosInterface;
	if constexpr (SweepOrRay == ESweepOrRay::Raycast)
	{
		return TraceCommonImp<FRaycastHit, FPTRaycastHit, SweepOrRay, SingleMultiOrTest>(ThreadContext, World, OutHits, GeomInputs, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
	}
	else
	{
		return TraceCommonImp<FSweepHit, FPTSweepHit, SweepOrRay, SingleMultiOrTest>(ThreadContext, World, OutHits, GeomInputs, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
	}
}

template <ESingleMultiOrTest SingleMultiOrTest, typename GeomInputsType, typename OutHitsType>
bool RaycastWithQueryHandler(UPhysicsQueryHandler& QueryHandler, const EThreadQueryContext ThreadContext, const UWorld* World, OutHitsType& OutHits, const GeomInputsType& GeomInputs, const FVector Start, const FVector End, const Chaos::FCommonQueryData& CommonData)
{
	static_assert(std::is_same_v<GeomInputsType, FRaycastSQAdditionalInputs>, "Invalid GeomInputs passed to Raycast Query.");

	Chaos::FRayQueryData RayData
	{
		.Start = Start,
		.End = End,
	};
	if constexpr (SingleMultiOrTest == ESingleMultiOrTest::Test)
	{
		return QueryHandler.RaycastTest(ThreadContext, World, RayData, CommonData, OutHits);
	}
	else if constexpr (SingleMultiOrTest == ESingleMultiOrTest::Single)
	{
		return QueryHandler.RaycastSingle(ThreadContext, World, RayData, CommonData, OutHits);
	}
	else
	{
		static_assert(SingleMultiOrTest == ESingleMultiOrTest::Multi);
		return QueryHandler.RaycastMulti(ThreadContext, World, RayData, CommonData, OutHits);
	}
}

template <ESingleMultiOrTest SingleMultiOrTest, typename OutHitsType>
bool SweepWithQueryHandler(UPhysicsQueryHandler& QueryHandler, const EThreadQueryContext ThreadContext, const UWorld* World, OutHitsType& OutHits, const Chaos::FQueryShape& QueryShape, const FQuat& GeomRot, const FVector Start, const FVector End, const Chaos::FCommonQueryData& CommonData)
{
	Chaos::FSweepQueryData SweepData
	{
		.Start = Start,
		.End = End,
		.QueryShape = QueryShape,
		.GeomRot = GeomRot,
	};
	if constexpr (SingleMultiOrTest == ESingleMultiOrTest::Test)
	{
		return QueryHandler.SweepTest(ThreadContext, World, SweepData, CommonData, OutHits);
	}
	else if constexpr (SingleMultiOrTest == ESingleMultiOrTest::Single)
	{
		return QueryHandler.SweepSingle(ThreadContext, World, SweepData, CommonData, OutHits);
	}
	else
	{
		static_assert(SingleMultiOrTest == ESingleMultiOrTest::Multi);
		return QueryHandler.SweepMulti(ThreadContext, World, SweepData, CommonData, OutHits);
	}
}

template <ESingleMultiOrTest SingleMultiOrTest, typename OutHitsType>
bool SweepWithQueryHandler(UPhysicsQueryHandler& QueryHandler, const EThreadQueryContext ThreadContext, const UWorld* World, OutHitsType& OutHits, const FGeomSQAdditionalInputs& GeomInputs, const FVector Start, const FVector End, const Chaos::FCommonQueryData& CommonData)
{
	Chaos::FQueryShape QueryShape;
	if (BuildQueryShape(GeomInputs.CollisionShape, QueryShape))
	{
		return SweepWithQueryHandler<SingleMultiOrTest>(QueryHandler, ThreadContext, World, OutHits, QueryShape, *GeomInputs.GetGeometryOrientation(), Start, End, CommonData);
	}
	return false;
}

template <ESingleMultiOrTest SingleMultiOrTest, typename OutHitsType>
bool SweepWithQueryHandler(UPhysicsQueryHandler& QueryHandler, const EThreadQueryContext ThreadContext, const UWorld* World, OutHitsType& OutHits, const FGeomCollectionSQAdditionalInputs& GeomInputs, const FVector Start, const FVector End, const Chaos::FCommonQueryData& CommonData)
{
	Chaos::FQueryShape QueryShape;
	if (BuildQueryShape(GeomInputs.Collection, QueryShape))
	{
		return SweepWithQueryHandler<SingleMultiOrTest>(QueryHandler, ThreadContext, World, OutHits, QueryShape, *GeomInputs.GetGeometryOrientation(), Start, End, CommonData);
	}
	return false;
}

template <ESweepOrRay SweepOrRay, ESingleMultiOrTest SingleMultiOrTest, typename OutHitsType, typename GeomInputsType, typename AccelContainerType = FDefaultAccelContainer>
bool TSceneCastCommon(const UWorld* World, OutHitsType& OutHits, const GeomInputsType& GeomInputs, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const AccelContainerType& AccelContainer = FDefaultAccelContainer{})
{
	if ((World == NULL) || (World->GetPhysicsScene() == NULL))
	{
		return false;
	}

	const EThreadQueryContext ThreadContext = GetThreadQueryContext(*World->GetPhysicsScene()->GetSolver());
	// If there's an acceleration structure override, always do a local query as the query handler is only needed for scene queries.
	if constexpr (AccelContainerType::HasAccelerationStructureOverride())
	{
		return TraceCommonImp<SweepOrRay, SingleMultiOrTest>(ThreadContext, World, OutHits, GeomInputs, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
	}
	else
	{
		UPhysicsQueryHandler* QueryHandler = World->PhysicsQueryHandler;
		if (GetIgnoreQueryHandler() || QueryHandler == nullptr)
		{
			return TraceCommonImp<SweepOrRay, SingleMultiOrTest>(ThreadContext, World, OutHits, GeomInputs, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
		}
		else
		{
			Chaos::FCommonQueryData CommonData
			{
				.TraceChannel = TraceChannel,
				.Params = Params,
				.ResponseParams = ResponseParams,
				.ObjectParams = ObjectParams,
			};
			if constexpr (SweepOrRay == ESweepOrRay::Raycast)
			{
				return RaycastWithQueryHandler<SingleMultiOrTest>(*QueryHandler, ThreadContext, World, OutHits, GeomInputs, Start, End, CommonData);
			}
			else
			{
				return SweepWithQueryHandler<SingleMultiOrTest>(*QueryHandler, ThreadContext, World, OutHits, GeomInputs, Start, End, CommonData);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// RAYCAST

bool FGenericPhysicsInterface::RaycastTest(const UWorld* World, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_RaycastAny);
	CSV_SCOPED_TIMING_STAT(SceneQuery, RaycastTest);

	FHitResult DummyHit(NoInit);

	return TSceneCastCommon<ESweepOrRay::Raycast, ESingleMultiOrTest::Test>(World, DummyHit, FRaycastSQAdditionalInputs(), Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<typename AccelType>
bool FGenericRaycastPhysicsInterfaceUsingSpatialAcceleration<AccelType>::RaycastTest(const AccelType& Accel, const UWorld* World, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_RaycastAny);
	CSV_SCOPED_TIMING_STAT(SceneQuery, RaycastTest);

	FHitResult DummyHit(NoInit);

	return TSceneCastCommon<ESweepOrRay::Raycast, ESingleMultiOrTest::Test>(World, DummyHit, FRaycastSQAdditionalInputs(), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}

bool FGenericPhysicsInterface::RaycastSingle(const UWorld* World, struct FHitResult& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_RaycastSingle);
	CSV_SCOPED_TIMING_STAT(SceneQuery, RaycastSingle);

	return TSceneCastCommon<ESweepOrRay::Raycast, ESingleMultiOrTest::Single>(World, OutHit, FRaycastSQAdditionalInputs(), Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<typename AccelType>
bool FGenericRaycastPhysicsInterfaceUsingSpatialAcceleration<AccelType>::RaycastSingle(const AccelType& Accel, const UWorld* World, struct FHitResult& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_RaycastSingle);
	CSV_SCOPED_TIMING_STAT(SceneQuery, RaycastSingle);

	return TSceneCastCommon<ESweepOrRay::Raycast, ESingleMultiOrTest::Single>(World, OutHit, FRaycastSQAdditionalInputs(), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}


bool FGenericPhysicsInterface::RaycastMulti(const UWorld* World, TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_RaycastMultiple);
	CSV_SCOPED_TIMING_STAT(SceneQuery, RaycastMultiple);

	return TSceneCastCommon<ESweepOrRay::Raycast, ESingleMultiOrTest::Multi>(World, OutHits, FRaycastSQAdditionalInputs(), Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<typename AccelType>
bool FGenericRaycastPhysicsInterfaceUsingSpatialAcceleration<AccelType>::RaycastMulti(const AccelType& Accel, const UWorld* World, TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_RaycastMultiple);
	CSV_SCOPED_TIMING_STAT(SceneQuery, RaycastMultiple);

	return TSceneCastCommon<ESweepOrRay::Raycast, ESingleMultiOrTest::Multi>(World, OutHits, FRaycastSQAdditionalInputs(), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}

//////////////////////////////////////////////////////////////////////////
// GEOM SWEEP

bool FGenericPhysicsInterface::GeomSweepTest(const UWorld* World, const struct FCollisionShape& CollisionShape, const FQuat& Rot, FVector Start, FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepAny);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepTest);

	FHitResult DummyHit(NoInit);

	return TSceneCastCommon<ESweepOrRay::Sweep, ESingleMultiOrTest::Test>(World, DummyHit, FGeomSQAdditionalInputs(CollisionShape, Rot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<typename AccelType, typename GeomType>
bool FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<AccelType, GeomType>::GeomSweepTest(const AccelType& Accel, const UWorld* World, const GeomType& InGeom, const FQuat& Rot, FVector Start, FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepAny);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepTest);

	FHitResult DummyHit(NoInit);

	return TSceneCastCommon<ESweepOrRay::Sweep, ESingleMultiOrTest::Test>(World, DummyHit, GeomToSQInputs(InGeom, Rot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}

template<>
bool FGenericPhysicsInterface::GeomSweepSingle(const UWorld* World, const FPhysicsGeometryCollection& InGeom, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepSingle);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepSingle);

	return TSceneCastCommon<ESweepOrRay::Sweep, ESingleMultiOrTest::Single>(World, OutHit, FGeomCollectionSQAdditionalInputs(InGeom, Rot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<>
bool FGenericPhysicsInterface::GeomSweepSingle(const UWorld* World, const struct FCollisionShape& CollisionShape, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepSingle);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepSingle);

	return TSceneCastCommon<ESweepOrRay::Sweep, ESingleMultiOrTest::Single>(World, OutHit, FGeomSQAdditionalInputs(CollisionShape, Rot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<>
bool FGenericPhysicsInterface::GeomSweepSingle(const UWorld* World, const FPhysicsGeometry& InGeom, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	return GeomSweepSingle(World, FChaosEngineInterface::GetGeometryCollection(InGeom), Rot, OutHit, Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<>
bool FGenericPhysicsInterface::GeomSweepMulti(const UWorld* World, const FPhysicsGeometryCollection& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepMultiple);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepMultiple);

	return TSceneCastCommon<ESweepOrRay::Sweep, ESingleMultiOrTest::Multi>(World, OutHits, FGeomCollectionSQAdditionalInputs(InGeom, InGeomRot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<>
bool FGenericPhysicsInterface::GeomSweepMulti(const UWorld* World, const FCollisionShape& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepMultiple);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepMultiple);

	return TSceneCastCommon<ESweepOrRay::Sweep, ESingleMultiOrTest::Multi>(World, OutHits, FGeomSQAdditionalInputs(InGeom, InGeomRot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<>
bool FGenericPhysicsInterface::GeomSweepMulti(const UWorld* World, const FPhysicsGeometry& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	return GeomSweepMulti(World, FChaosEngineInterface::GetGeometryCollection(InGeom), InGeomRot, OutHits, Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
}

//////////////////////////////////////////////////////////////////////////
// GEOM OVERLAP

using EQueryInfo = Chaos::EQueryInfo;

template <typename OverlapHitType, EQueryInfo InfoType, typename CollisionAnalyzerType, typename AccelContainerType>
bool GeomOverlapMultiImp(const UWorld* World, const FPhysicsGeometry& Geom, const CollisionAnalyzerType& CollisionAnalyzerShape, const FTransform& GeomPose, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const AccelContainerType& AccelContainer)
{
	using namespace ChaosInterface;

	FScopeCycleCounter Counter(Params.StatId);

	if ((World == NULL) || (World->GetPhysicsScene() == NULL))
	{
		return false;
	}

	STARTQUERYTIMER();

	bool bHaveBlockingHit = false;

	// overlapMultiple only supports sphere/capsule/box
	const ECollisionShapeType GeomType = GetType(Geom);
	if (GeomType == ECollisionShapeType::Sphere || GeomType == ECollisionShapeType::Capsule || GeomType == ECollisionShapeType::Box || GeomType == ECollisionShapeType::Convex)
	{
		constexpr bool bIsRetryQuery = false;
		CVD_TRACE_SCOPED_SCENE_QUERY_HELPER(World, &Geom, GeomPose, FVector::ZeroVector, TraceChannel, Params, ResponseParams, ObjectParams, EChaosVDSceneQueryType::Overlap, (InfoType == EQueryInfo::GatherAll) ? EChaosVDSceneQueryMode::Multi : EChaosVDSceneQueryMode::Test, bIsRetryQuery);

		// Create filter data used to filter collisions
		const Chaos::Filter::FQueryFilterData QueryFilterData = CreateChaosQueryFilterData(TraceChannel, Params.bTraceComplex, ResponseParams.CollisionResponse, Params, ObjectParams, InfoType != EQueryInfo::IsAnything);
		FCollisionQueryFilterCallback QueryCallback(Params, false);
		QueryCallback.bIgnoreTouches |= (InfoType == EQueryInfo::IsBlocking); // pre-filter to ignore touches and only get blocking hits, if that's what we're after.
		QueryCallback.bIsOverlapQuery = true;

		EQueryFlags QueryFlags = InfoType == EQueryInfo::GatherAll ? EQueryFlags::PreFilter : (EQueryFlags::PreFilter | EQueryFlags::AnyHit);
		if (Params.bSkipNarrowPhase)
		{
			QueryFlags = QueryFlags | EQueryFlags::SkipNarrowPhase;
		}
		FDynamicHitBuffer<OverlapHitType> OverlapBuffer;

		FQueryDebugParams DebugParams;
#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
		DebugParams.bDebugQuery = Params.bDebugQuery;
#endif

		// Enable scene locks, in case they are required
		FPhysScene& PhysScene = *World->GetPhysicsScene();

		FPhysicsCommand::ExecuteRead(&PhysScene, [&]()
		{
			const FSceneQueryCommonParams CommonParams(QueryCallback, QueryFilterData, QueryFlags, DebugParams);
			{
				FScopedSQHitchRepeater<typename TRemoveReference<decltype(OverlapBuffer)>::Type> HitchRepeater(OverlapBuffer, QueryCallback, FHitchDetectionInfo(GeomPose, TraceChannel, Params));
				do
				{
					if constexpr (AccelContainerType::HasAccelerationStructureOverride())
					{
						Chaos::Private::LowLevelOverlap(AccelContainer.GetSpatialAcceleration(), Geom, GeomPose, HitchRepeater.GetBuffer(), CommonParams);
					}
					else
					{
						Chaos::Private::LowLevelOverlap(PhysScene, Geom, GeomPose, HitchRepeater.GetBuffer(), CommonParams);
					}
				} while(HitchRepeater.RepeatOnHitch());

				if(GetHasBlock(OverlapBuffer) && InfoType != EQueryInfo::GatherAll)	//just want true or false so don't bother gathering info
				{
					bHaveBlockingHit = true;
				}
			}

			if(InfoType == EQueryInfo::GatherAll)	//if we are gathering all we need to actually convert to UE format
			{
				const int32 NumHits = OverlapBuffer.GetNumHits();

				if(NumHits > 0)
				{
					bHaveBlockingHit = ConvertOverlapResults(NumHits, OverlapBuffer.GetHits(), QueryFilterData, OutOverlaps);

					// This block is only necessary on the game thread when dealing with cluster unions.
					// TODO: Is there a way to get this to generalize better to the PT as well? Right now it depends on GT functions on the cluster union component.
					if constexpr (std::is_base_of_v<ChaosInterface::FActorShape, OverlapHitType>)
					{
						auto DoClusterUnionOverlap = [&GeomPose, &Geom, TraceChannel, &Params, &ResponseParams, &ObjectParams](const FOverlapResult& OriginalOverlap, TArray<FOverlapResult>& NewOverlaps)
						{
							FClusterUnionHit Result;

#if !UE_WITH_REMOTE_OBJECT_HANDLE
							if (UClusterUnionComponent* ClusterUnion = Cast<UClusterUnionComponent>(OriginalOverlap.GetComponent()))
							{
								Result.bIsClusterUnion = true;
								ClusterUnion->OverlapComponentWithResult(GeomPose.GetTranslation(), GeomPose.GetRotation(), Geom, TraceChannel, Params, ResponseParams, ObjectParams, NewOverlaps);
								Result.bHit = !NewOverlaps.IsEmpty();
							}
#endif

							return Result;
						};

						if (!OutOverlaps.IsEmpty() && Params.bTraceIntoSubComponents)
						{
							TArray<FOverlapResult> AllNewOverlaps;
							TArray<int32> ClusterUnionIndices;

							for (int32 Index = 0; Index < OutOverlaps.Num(); ++Index)
							{
								TArray<FOverlapResult> NewOverlaps;
								FClusterUnionHit ClusterUnionHit = DoClusterUnionOverlap(OutOverlaps[Index], NewOverlaps);
								if (ClusterUnionHit.bIsClusterUnion)
								{
									if (Params.bReplaceHitWithSubComponents || !ClusterUnionHit.bHit)
									{
										ClusterUnionIndices.Add(Index);
									}

									if (ClusterUnionHit.bHit)
									{
										AllNewOverlaps.Append(NewOverlaps);
									}
								}
							}

							for (int32 Index = ClusterUnionIndices.Num() - 1; Index >= 0; --Index)
							{
								// No shrinking since we're going to be adding more elements shortly.
								OutOverlaps.RemoveAtSwap(ClusterUnionIndices[Index], EAllowShrinking::No);
							}

							if (Params.bReplaceHitWithSubComponents)
							{
								OutOverlaps.Append(AllNewOverlaps);
							}
							bHaveBlockingHit &= !OutOverlaps.IsEmpty();
						}
					}
				}
			}

		});
	}
	else
	{
		UE_LOGF(LogCollision, Log, "GeomOverlapMulti : unsupported shape - only supports sphere, capsule, box");
	}

#if ENABLE_COLLISION_ANALYZER
	if (GCollisionAnalyzerIsRecording)
	{
		// Determine query mode ('single' doesn't really exist for overlaps)
		ECAQueryMode::Type QueryMode = (InfoType == EQueryInfo::GatherAll) ? ECAQueryMode::Multi : ECAQueryMode::Test;

		CAPTUREGEOMOVERLAP(World, CollisionAnalyzerShape, GeomPose, QueryMode, TraceChannel, Params, ResponseParams, ObjectParams, OutOverlaps);
	}
#endif // ENABLE_COLLISION_ANALYZER

	return bHaveBlockingHit;
}

template <EQueryInfo InfoType, typename GeomType, typename AccelContainerType = FDefaultAccelContainer>
bool GeomOverlapMultiHelper(const EThreadQueryContext ThreadContext, const UWorld* World, const GeomType& InGeom, const FTransform& GeomPose, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const AccelContainerType& AccelContainer = FDefaultAccelContainer{})
{
	using namespace ChaosInterface;

	TGeomSQInputs<GeomType> SQInputs = GeomToSQInputs(InGeom, GeomPose.GetRotation());
	const FPhysicsGeometry& Geom = *SQInputs.GetGeometry();
	auto& CollisionAnalyzerShape = *SQInputs.GetCollisionShape();
	if (ThreadContext == EThreadQueryContext::GTData)
	{
		return GeomOverlapMultiImp<FOverlapHit, InfoType>(World, Geom, CollisionAnalyzerShape, GeomPose, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
	}
	else
	{
		return GeomOverlapMultiImp<FPTOverlapHit, InfoType>(World, Geom, CollisionAnalyzerShape, GeomPose, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
	}
}

template <EQueryInfo InfoType>
bool GeomOverlapMultiHelperWithQueryHandler(UPhysicsQueryHandler& QueryHandler, const EThreadQueryContext ThreadContext, const UWorld* World, const Chaos::FQueryShape& QueryShape, const FTransform& GeomPose, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	Chaos::FCommonQueryData CommonData
	{
		.TraceChannel = TraceChannel,
		.Params = Params,
		.ResponseParams = ResponseParams,
		.ObjectParams = ObjectParams,
	};
	Chaos::FOverlapQueryData OverlapData
	{
		.QueryShape = QueryShape,
		.GeomPose = GeomPose,
	};
	return QueryHandler.Overlap(InfoType, ThreadContext, World, OverlapData, CommonData, OutOverlaps);
}

template <EQueryInfo InfoType>
bool GeomOverlapMultiHelperWithQueryHandler(UPhysicsQueryHandler& QueryHandler, const EThreadQueryContext ThreadContext, const UWorld* World, const FCollisionShape& CollisionShape, const FTransform& GeomPose, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	Chaos::FQueryShape QueryShape;
	if (BuildQueryShape(CollisionShape, QueryShape))
	{
		return GeomOverlapMultiHelperWithQueryHandler<InfoType>(QueryHandler, ThreadContext, World, QueryShape, GeomPose, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams);
	}
	return false;
}

template <EQueryInfo InfoType>
bool GeomOverlapMultiHelperWithQueryHandler(UPhysicsQueryHandler& QueryHandler, const EThreadQueryContext ThreadContext, const UWorld* World, const FPhysicsGeometryCollection& CollisionShape, const FTransform& GeomPose, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	Chaos::FQueryShape QueryShape;
	if (BuildQueryShape(CollisionShape, QueryShape))
	{
		return GeomOverlapMultiHelperWithQueryHandler<InfoType>(QueryHandler, ThreadContext, World, QueryShape, GeomPose, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams);
	}
	return false;
}

template <EQueryInfo InfoType, typename GeomType, typename AccelContainerType = FDefaultAccelContainer>
bool GeomOverlapMultiHelper(const UWorld* World, const GeomType& InGeom, const FTransform& GeomPose, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const AccelContainerType& AccelContainer = FDefaultAccelContainer{})
{
	using namespace ChaosInterface;

	if ((World == NULL) || (World->GetPhysicsScene() == NULL))
	{
		return false;
	}

	const EThreadQueryContext ThreadContext = GetThreadQueryContext(*World->GetPhysicsScene()->GetSolver());
	if constexpr (AccelContainerType::HasAccelerationStructureOverride())
	{
		return GeomOverlapMultiHelper<InfoType>(ThreadContext, World, InGeom, GeomPose, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
	}
	else
	{
		UPhysicsQueryHandler* QueryHandler = World->PhysicsQueryHandler;
		if (GetIgnoreQueryHandler() || QueryHandler == nullptr)
		{
			return GeomOverlapMultiHelper<InfoType>(ThreadContext, World, InGeom, GeomPose, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams, AccelContainer);
		}
		else
		{
			return GeomOverlapMultiHelperWithQueryHandler<InfoType>(*QueryHandler, ThreadContext, World, InGeom, GeomPose, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams);
		}
	}
}

bool FGenericPhysicsInterface::GeomOverlapBlockingTest(const UWorld* World, const struct FCollisionShape& CollisionShape, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomOverlapBlocking);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomOverlapBlocking);

	TArray<FOverlapResult> Overlaps;	//needed only for template shared code
	FTransform GeomTransform(Rot, Pos);
	return GeomOverlapMultiHelper<EQueryInfo::IsBlocking>(World, CollisionShape, GeomTransform, Overlaps, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<typename AccelType, typename GeomType>
bool FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<AccelType, GeomType>::GeomOverlapBlockingTest(const AccelType& Accel, const UWorld* World, const GeomType& InGeom, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomOverlapBlocking);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomOverlapBlocking);

	TArray<FOverlapResult> Overlaps;	//needed only for template shared code
	FTransform GeomTransform{ Rot, Pos };
	return GeomOverlapMultiHelper<EQueryInfo::IsBlocking>(World, InGeom, GeomTransform, Overlaps, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}

bool FGenericPhysicsInterface::GeomOverlapAnyTest(const UWorld* World, const struct FCollisionShape& CollisionShape, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomOverlapAny);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomOverlapAny);

	TArray<FOverlapResult> Overlaps;	//needed only for template shared code
	FTransform GeomTransform(Rot, Pos);
	return GeomOverlapMultiHelper<EQueryInfo::IsAnything>(World, CollisionShape, GeomTransform, Overlaps, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<typename AccelType, typename GeomType>
bool FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<AccelType, GeomType>::GeomOverlapAnyTest(const AccelType& Accel, const UWorld* World, const GeomType& InGeom, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomOverlapAny);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomOverlapAny);

	TArray<FOverlapResult> Overlaps;	//needed only for template shared code
	FTransform GeomTransform{ Rot, Pos };
	return GeomOverlapMultiHelper<EQueryInfo::IsAnything>(World, InGeom, GeomTransform, Overlaps, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}

template<>
bool FGenericPhysicsInterface::GeomOverlapMulti(const UWorld* World, const FPhysicsGeometryCollection& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomOverlapMultiple);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomOverlapMultiple);

	FTransform GeomTransform(InRotation, InPosition);
	return GeomOverlapMultiHelper<EQueryInfo::GatherAll>(World, InGeom, GeomTransform, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<>
bool FGenericPhysicsInterface::GeomOverlapMulti(const UWorld* World, const FCollisionShape& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomOverlapMultiple);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomOverlapMultiple);

	FTransform GeomTransform(InRotation, InPosition);
	return GeomOverlapMultiHelper<EQueryInfo::GatherAll>(World, InGeom, GeomTransform, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<>
bool FGenericPhysicsInterface::GeomOverlapMulti(const UWorld* World, const FPhysicsGeometry& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	return GeomOverlapMulti(World, FChaosEngineInterface::GetGeometryCollection(InGeom), InPosition, InRotation, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams);
}

template<typename AccelType, typename GeomType>
bool FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<AccelType, GeomType>::GeomSweepSingle(const AccelType& Accel, const UWorld* World, const GeomType& InGeom, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepSingle);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepSingle);

	return TSceneCastCommon<ESweepOrRay::Sweep, ESingleMultiOrTest::Single>(World, OutHit, GeomToSQInputs(InGeom, Rot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}

template<typename AccelType, typename GeomType>
bool FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<AccelType, GeomType>::GeomSweepMulti(const AccelType& Accel, const UWorld* World, const GeomType& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	using namespace ChaosInterface;

	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepMultiple);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepMultiple);

	return TSceneCastCommon<ESweepOrRay::Sweep, ESingleMultiOrTest::Multi>(World, OutHits, GeomToSQInputs(InGeom, InGeomRot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}

template<typename AccelType, typename GeomType>
bool FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<AccelType, GeomType>::GeomOverlapMulti(const AccelType& Accel, const UWorld* World, const GeomType& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomOverlapMultiple);
	CSV_SCOPED_TIMING_STAT(SceneQuery, GeomOverlapMultiple);

	FTransform GeomTransform{ InRotation, InPosition };
	return GeomOverlapMultiHelper<EQueryInfo::GatherAll>(World, InGeom, GeomTransform, OutOverlaps, TraceChannel, Params, ResponseParams, ObjectParams, FOverrideAccelContainer{ Accel });
}

template struct FGenericRaycastPhysicsInterfaceUsingSpatialAcceleration<Chaos::IDefaultChaosSpatialAcceleration>;
template struct FGenericRaycastPhysicsInterfaceUsingSpatialAcceleration<IExternalSpatialAcceleration>;

#define DEFINE_GENERIC_GEOM_FOR_ACCEL(TACCEL) \
template struct FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<TACCEL, FCollisionShape>;\
template struct FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<TACCEL, FPhysicsGeometry>;\
template struct FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<TACCEL, FPhysicsGeometryCollection>;

DEFINE_GENERIC_GEOM_FOR_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DEFINE_GENERIC_GEOM_FOR_ACCEL(IExternalSpatialAcceleration)

namespace Chaos::Private
{
	bool FGenericPhysicsInterface_Internal::SpherecastMulti(const UWorld* World, float QueryRadius, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
	{
		SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
		SCOPE_CYCLE_COUNTER(STAT_Collision_SpherecastMultiple_Internal);
		CSV_SCOPED_TIMING_STAT(SceneQuery, SpherecastMultiple_Internal);

		Chaos::EnsureIsInPhysicsThreadContext();

		if ((World == NULL) || (World->GetPhysicsScene() == NULL))
		{
			return false;
		}

		if (QueryRadius > UE_SMALL_NUMBER)
		{
			using PTTraits = TSQTraits<ChaosInterface::FPTSweepHit, ESweepOrRay::Sweep, ESingleMultiOrTest::Multi>;
			return TSceneCastCommonImp<PTTraits, FGeomSQAdditionalInputs>(World, OutHits, FGeomSQAdditionalInputs(FCollisionShape::MakeSphere(QueryRadius), FQuat::Identity), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FDefaultAccelContainer{});
		}
		else
		{
			using PTTraits = TSQTraits<ChaosInterface::FPTRaycastHit, ESweepOrRay::Raycast, ESingleMultiOrTest::Multi>;
			return TSceneCastCommonImp<PTTraits, FRaycastSQAdditionalInputs>(World, OutHits, FRaycastSQAdditionalInputs(), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FDefaultAccelContainer{});
		}
	}

	bool FGenericPhysicsInterface_Internal::GeomSweepMulti(const UWorld* World, const FCollisionShape& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
	{
		if (InGeom.IsNearlyZero())
		{
			// Fall back to a raycast
			return RaycastMulti(World, OutHits, Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
		}

		SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
		SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepMultiple_Internal);
		CSV_SCOPED_TIMING_STAT(SceneQuery, GeomSweepMultiple_Internal);

		Chaos::EnsureIsInPhysicsThreadContext();

		if ((World == NULL) || (World->GetPhysicsScene() == NULL))
		{
			return false;
		}

		using PTTraits = TSQTraits<ChaosInterface::FPTSweepHit, ESweepOrRay::Sweep, ESingleMultiOrTest::Multi>;
		return TSceneCastCommonImp<PTTraits, FGeomSQAdditionalInputs>(World, OutHits, FGeomSQAdditionalInputs(InGeom, InGeomRot), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FDefaultAccelContainer{});
	}

	bool FGenericPhysicsInterface_Internal::RaycastMulti(const UWorld* World, TArray<FHitResult>& OutHits, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams)
	{
		SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
		SCOPE_CYCLE_COUNTER(STAT_Collision_RaycastMultiple_Internal);
		CSV_SCOPED_TIMING_STAT(SceneQuery, RaycastMultiple_Internal);

		Chaos::EnsureIsInPhysicsThreadContext();

		if ((World == NULL) || (World->GetPhysicsScene() == NULL))
		{
			return false;
		}

		using PTTraits = TSQTraits<ChaosInterface::FPTRaycastHit, ESweepOrRay::Raycast, ESingleMultiOrTest::Multi>;
		return TSceneCastCommonImp<PTTraits, FRaycastSQAdditionalInputs>(World, OutHits, FRaycastSQAdditionalInputs(), Start, End, TraceChannel, Params, ResponseParams, ObjectParams, FDefaultAccelContainer{});
	}

	template <typename OverlapHitType, EQueryInfo InfoType>
	bool OverlapImpl(const UWorld* World, const Chaos::FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData, TArray<FOverlapResult>& OutOverlaps)
	{
		const FQuat& Rotation = OverlapData.GeomPose.GetRotation();
		if (OverlapData.QueryShape.IsConvexShape())
		{
			FImplicitObjectPtr Convex;
			ConvertToConvex(OverlapData.QueryShape, Convex);
			FPhysicsGeometryCollection CollisionShape = FChaosEngineInterface::GetGeometryCollection(*Convex.GetReference());
			const FPhysicsGeometry& Geometry = CollisionShape.GetGeometry();

			return GeomOverlapMultiImp<OverlapHitType, InfoType>(World, Geometry, CollisionShape, OverlapData.GeomPose, OutOverlaps, CommonData.TraceChannel, CommonData.Params, CommonData.ResponseParams, CommonData.ObjectParams, FDefaultAccelContainer{});
		}
		else
		{
			FPhysicsShapeAdapter Adaptor(Rotation, OverlapData.QueryShape.CollisionShape);
			const FPhysicsGeometry& Geom = Adaptor.GetGeometry();
			const FCollisionShape& CollisionShape = OverlapData.QueryShape.CollisionShape;

			return GeomOverlapMultiImp<OverlapHitType, InfoType>(World, Geom, CollisionShape, OverlapData.GeomPose, OutOverlaps, CommonData.TraceChannel, CommonData.Params, CommonData.ResponseParams, CommonData.ObjectParams, FDefaultAccelContainer{});
		}
	}

	template <EQueryInfo InfoType, typename ... ArgTypes>
	bool ForwardThreadContextType_Overlap(const EThreadQueryContext ThreadContext, ArgTypes&& ... Args)
	{
		using namespace ChaosInterface;
		if (ThreadContext == EThreadQueryContext::GTData)
		{
			return OverlapImpl<FOverlapHit, InfoType>(std::forward<ArgTypes>(Args)...);
		}
		else
		{
			return OverlapImpl<FPTOverlapHit, InfoType>(std::forward<ArgTypes>(Args)...);
		}
	}

	bool FQueryInterface_Internal::Overlap(const EQueryInfo InfoType, const EThreadQueryContext ThreadContext, const UWorld* World, const FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData, TArray<FOverlapResult>& OutOverlaps)
	{
		if (InfoType == EQueryInfo::GatherAll)
		{
			return ForwardThreadContextType_Overlap<EQueryInfo::GatherAll>(ThreadContext, World, OverlapData, CommonData, OutOverlaps);
		}
		else if (InfoType == EQueryInfo::IsBlocking)
		{
			return ForwardThreadContextType_Overlap<EQueryInfo::IsBlocking>(ThreadContext, World, OverlapData, CommonData, OutOverlaps);
		}
		else if (InfoType == EQueryInfo::IsAnything)
		{
			return ForwardThreadContextType_Overlap<EQueryInfo::IsAnything>(ThreadContext, World, OverlapData, CommonData, OutOverlaps);
		}
		return false;
	}

	template <ESingleMultiOrTest SingleMultiOrTest, typename HitResultsType>
	bool RaycastCommon(const EThreadQueryContext ThreadContext, const UWorld* World, const Chaos::FRayQueryData& RayData, const Chaos::FCommonQueryData& CommonData, HitResultsType& OutHits)
	{
		FRaycastSQAdditionalInputs GeomInputs;
		return TraceCommonImp<ESweepOrRay::Raycast, SingleMultiOrTest>(ThreadContext, World, OutHits, GeomInputs, RayData.Start, RayData.End, CommonData.TraceChannel, CommonData.Params, CommonData.ResponseParams, CommonData.ObjectParams, FDefaultAccelContainer{});
	}

	bool FQueryInterface_Internal::RaycastTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits)
	{
		return RaycastCommon<ESingleMultiOrTest::Test>(ThreadContext, World, RayData, CommonData, OutHits);
	}

	bool FQueryInterface_Internal::RaycastSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits)
	{
		return RaycastCommon<ESingleMultiOrTest::Single>(ThreadContext, World, RayData, CommonData, OutHits);
	}

	bool FQueryInterface_Internal::RaycastMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits)
	{
		return RaycastCommon<ESingleMultiOrTest::Multi>(ThreadContext, World, RayData, CommonData, OutHits);
	}

	template <ESingleMultiOrTest SingleMultiOrTest, typename HitResultsType>
	bool SweepCommon(const EThreadQueryContext ThreadContext, const UWorld* World, const Chaos::FSweepQueryData& SweepData, const Chaos::FCommonQueryData& CommonData, HitResultsType& OutHits)
	{
		if (SweepData.QueryShape.IsConvexShape())
		{
			Chaos::FImplicitObjectPtr Convex;
			ConvertToConvex(SweepData.QueryShape, Convex);
			FPhysicsGeometryCollection GeomCollection = FChaosEngineInterface::GetGeometryCollection(*Convex.GetReference());

			FGeomCollectionSQAdditionalInputs GeomInputs(GeomCollection, SweepData.GeomRot);
			return TraceCommonImp<ESweepOrRay::Sweep, SingleMultiOrTest>(ThreadContext, World, OutHits, GeomInputs, SweepData.Start, SweepData.End, CommonData.TraceChannel, CommonData.Params, CommonData.ResponseParams, CommonData.ObjectParams, FDefaultAccelContainer{});
		}
		else
		{
			FGeomSQAdditionalInputs GeomInputs(SweepData.QueryShape.CollisionShape, SweepData.GeomRot);
			return TraceCommonImp<ESweepOrRay::Sweep, SingleMultiOrTest>(ThreadContext, World, OutHits, GeomInputs, SweepData.Start, SweepData.End, CommonData.TraceChannel, CommonData.Params, CommonData.ResponseParams, CommonData.ObjectParams, FDefaultAccelContainer{});
		}
	}

	bool FQueryInterface_Internal::SweepTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits)
	{
		return SweepCommon<ESingleMultiOrTest::Test>(ThreadContext, World, SweepData, CommonData, OutHits);
	}

	bool FQueryInterface_Internal::SweepSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits)
	{
		return SweepCommon<ESingleMultiOrTest::Single>(ThreadContext, World, SweepData, CommonData, OutHits);
	}

	bool FQueryInterface_Internal::SweepMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits)
	{
		return SweepCommon<ESingleMultiOrTest::Multi>(ThreadContext, World, SweepData, CommonData, OutHits);
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	FScopedIgnoreQueryHandler::FScopedIgnoreQueryHandler(bool bIgnore)
	{
		bOldState = bIgnoreQueryHandler;
		bIgnoreQueryHandler = bIgnore;
	}

	FScopedIgnoreQueryHandler::~FScopedIgnoreQueryHandler()
	{
		bIgnoreQueryHandler = bOldState;
	}
#endif
}
