// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/ChaosInterfaceWrapper.h"
#include "SQAccelerator.h"
#include "SQVisitor.h"

#include "PhysTestSerializer.h"

#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/ExternalSpatialAccelerationPayload.h"

namespace
{
	/**
	 * This is meant as a replacement for FChaosSQAccelerator. There doesn't seem to be a need to expose this publicly since it'll just be used internally to allow
	 * us to use more than one type of acceleration structure payload.
	 */
	template<typename TPayload>
	class FGenericChaosSQAccelerator
	{
	public:
		explicit FGenericChaosSQAccelerator(const Chaos::ISpatialAcceleration<TPayload, Chaos::FReal, 3>& InSpatialAcceleration)
			: SpatialAcceleration(InSpatialAcceleration)
		{}

		template<typename THit>
		void Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<THit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
		{
			using namespace Chaos;
			using namespace ChaosInterface;

			TSQVisitor<TSphere<FReal, 3>, TPayload, THit, std::is_same_v<THit, FRaycastHit>> RaycastVisitor(Start, Dir, HitBuffer, OutputFlags, CommonParams);
			HitBuffer.IncFlushCount();
			SpatialAcceleration.Raycast(Start, Dir, DeltaMagnitude, RaycastVisitor);
			HitBuffer.DecFlushCount();
		}

		template<typename THit>
		void Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<THit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
		{
			return Chaos::Utilities::CastHelper(QueryGeom, StartTM, [&](const auto& Downcast, const FTransform& StartFullTM)
			{
				return SweepHelper(Downcast, SpatialAcceleration, StartFullTM, Dir, DeltaMagnitude, HitBuffer, OutputFlags, CommonParams);
			});
		}

		template<typename THit>
		void Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<THit>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
		{
			return Chaos::Utilities::CastHelper(QueryGeom, GeomPose, [&](const auto& Downcast, const FTransform& GeomFullPose)
			{
				return OverlapHelper(Downcast, SpatialAcceleration, GeomFullPose, HitBuffer, CommonParams);
			});
		}

	private:
		const Chaos::ISpatialAcceleration<TPayload, Chaos::FReal, 3>& SpatialAcceleration;
	};

	template<typename TAccelContainer, bool bGTData>
	struct FAccelerationContainerTraits
	{
		using TAccelStructure = std::conditional_t<std::is_same_v<TAccelContainer, FPhysScene>, Chaos::IDefaultChaosSpatialAcceleration, TAccelContainer>;

		static const TAccelStructure* GetSpatialAccelerationFromContainer(const TAccelContainer& Container)
		{
			if constexpr (IsPhysScene())
			{
				if constexpr (bGTData)
				{
					return Container.GetSpacialAcceleration();
				}
				else
				{
					return Container.GetSolver()->GetInternalAccelerationStructure_Internal();
				}
			}
			else
			{
				return &Container;
			}
		}

		static constexpr bool IsPhysScene()
		{
			return std::is_same_v<TAccelContainer, FPhysScene>;
		}
	};
}

template <typename TAccelContainer, typename THitRaycast>
void LowLevelRaycastImpl(const TAccelContainer& Container, const FVector& Start, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THitRaycast>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams)
{
	constexpr bool bGTData = std::is_same_v<THitRaycast, FHitRaycast>;
	using TTraits = FAccelerationContainerTraits<TAccelContainer, bGTData>;
	if (const typename TTraits::TAccelStructure* SolverAccelerationStructure = TTraits::GetSpatialAccelerationFromContainer(Container))
	{
		FGenericChaosSQAccelerator<typename TTraits::TAccelStructure::TPayload> SQAccelerator(*SolverAccelerationStructure);
		double Time = 0.0;
		{
			FScopedDurationTimer Timer(Time);
			SQAccelerator.Raycast(Start, Dir, DeltaMag, HitBuffer, OutputFlags, CommonParams);
		}
	}
}

template <typename TAccelContainer, typename THitSweep>
void LowLevelSweepImpl(const TAccelContainer& Container, const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THitSweep>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams)
{
	constexpr bool bGTData = std::is_same<THitSweep, FHitSweep>::value;
	using TTraits = FAccelerationContainerTraits<TAccelContainer, bGTData>;
	if (const typename TTraits::TAccelStructure* SolverAccelerationStructure = TTraits::GetSpatialAccelerationFromContainer(Container))
	{
		FGenericChaosSQAccelerator<typename TTraits::TAccelStructure::TPayload> SQAccelerator(*SolverAccelerationStructure);
		{
			double Time = 0.0;
			{
				FScopedDurationTimer Timer(Time);
				SQAccelerator.Sweep(QueryGeom, StartTM, Dir, DeltaMag, HitBuffer, OutputFlags, CommonParams);
			}
		}
	}
}

template <typename TAccelContainer, typename THitOverlap>
void LowLevelOverlapImpl(const TAccelContainer& Container, const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<THitOverlap>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams)
{
	constexpr bool bGTData = std::is_same<THitOverlap, FHitOverlap>::value;
	using TTraits = FAccelerationContainerTraits<TAccelContainer, bGTData>;
	if (const typename TTraits::TAccelStructure* SolverAccelerationStructure = TTraits::GetSpatialAccelerationFromContainer(Container))
	{
		FGenericChaosSQAccelerator<typename TTraits::TAccelStructure::TPayload> SQAccelerator(*SolverAccelerationStructure);
		double Time = 0.0;
		{
			FScopedDurationTimer Timer(Time);
			SQAccelerator.Overlap(QueryGeom, GeomPose, HitBuffer, CommonParams);
		}
	}
}

#define DEFINE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, THIT) template<> void Chaos::Private::LowLevelRaycast(const TACCEL& Container, const FVector& Start, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THIT>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) \
{ LowLevelRaycastImpl(Container, Start, Dir, DeltaMag, HitBuffer, OutputFlags, CommonParams); }
#define DEFINE_LOW_LEVEL_RAYCAST_ACCEL(TACCEL) \
DEFINE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, FHitRaycast); \
DEFINE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, ChaosInterface::FPTRaycastHit);
DEFINE_LOW_LEVEL_RAYCAST_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DEFINE_LOW_LEVEL_RAYCAST_ACCEL(IExternalSpatialAcceleration)
DEFINE_LOW_LEVEL_RAYCAST_ACCEL(FPhysScene)
#undef DEFINE_LOW_LEVEL_RAYCAST_ACCEL
#undef DEFINE_LOW_LEVEL_RAYCAST_ACCEL_HIT

#define DEFINE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, THIT) template<> void Chaos::Private::LowLevelSweep(const TACCEL& Container, const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THIT>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) \
{ LowLevelSweepImpl(Container, QueryGeom, StartTM, Dir, DeltaMag, HitBuffer, OutputFlags, CommonParams); }
#define DEFINE_LOW_LEVEL_SWEEP_ACCEL(TACCEL) \
DEFINE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, FHitSweep); \
DEFINE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, ChaosInterface::FPTSweepHit);
DEFINE_LOW_LEVEL_SWEEP_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DEFINE_LOW_LEVEL_SWEEP_ACCEL(IExternalSpatialAcceleration)
DEFINE_LOW_LEVEL_SWEEP_ACCEL(FPhysScene)
#undef DEFINE_LOW_LEVEL_SWEEP_ACCEL
#undef DEFINE_LOW_LEVEL_SWEEP_ACCEL_HIT

#define DEFINE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, THIT) template<> void Chaos::Private::LowLevelOverlap(const TACCEL& Container, const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<THIT>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams) \
{ LowLevelOverlapImpl(Container, QueryGeom, GeomPose, HitBuffer, CommonParams); }
#define DEFINE_LOW_LEVEL_OVERLAP_ACCEL(TACCEL) \
DEFINE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, FHitOverlap) \
DEFINE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, ChaosInterface::FPTOverlapHit)
DEFINE_LOW_LEVEL_OVERLAP_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DEFINE_LOW_LEVEL_OVERLAP_ACCEL(IExternalSpatialAcceleration)
DEFINE_LOW_LEVEL_OVERLAP_ACCEL(FPhysScene)
#undef DEFINE_LOW_LEVEL_OVERLAP_ACCEL
#undef DEFINE_LOW_LEVEL_OVERLAP_ACCEL_HIT



PRAGMA_DISABLE_DEPRECATION_WARNINGS
template <typename TAccelContainer, typename THitRaycast>
void LowLevelRaycastImpl(const TAccelContainer& Container, const FVector& Start, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THitRaycast>& HitBuffer, EHitFlags OutputFlags, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams)
{
	const ChaosInterface::FSceneQueryCommonParams CommonParams(*QueryCallback, QueryFilterData, DebugParams);
	LowLevelRaycastImpl(Container, Start, Dir, DeltaMag, HitBuffer, OutputFlags, CommonParams);
}

#define DEFINE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, THIT) template<> void Chaos::Private::LowLevelRaycast(const TACCEL& Container, const FVector& Start, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THIT>& HitBuffer, EHitFlags OutputFlags, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) \
{ LowLevelRaycastImpl(Container, Start, Dir, DeltaMag, HitBuffer, OutputFlags, QueryFlags, Filter, QueryFilterData, QueryCallback, DebugParams); }
#define DEFINE_LOW_LEVEL_RAYCAST_ACCEL(TACCEL) \
DEFINE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, FHitRaycast); \
DEFINE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, ChaosInterface::FPTRaycastHit);
DEFINE_LOW_LEVEL_RAYCAST_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DEFINE_LOW_LEVEL_RAYCAST_ACCEL(IExternalSpatialAcceleration)
DEFINE_LOW_LEVEL_RAYCAST_ACCEL(FPhysScene)

template <typename TAccelContainer, typename THitSweep>
void LowLevelSweepImpl(const TAccelContainer& Container, const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THitSweep>& HitBuffer, EHitFlags OutputFlags, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams)
{
	const ChaosInterface::FSceneQueryCommonParams CommonParams(*QueryCallback, QueryFilterData, DebugParams);
	LowLevelSweepImpl(Container, QueryGeom, StartTM, Dir, DeltaMag, HitBuffer, OutputFlags, CommonParams);
}

#define DEFINE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, THIT) template<> void Chaos::Private::LowLevelSweep(const TACCEL& Container, const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THIT>& HitBuffer, EHitFlags OutputFlags, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) \
{ LowLevelSweepImpl(Container, QueryGeom, StartTM, Dir, DeltaMag, HitBuffer, OutputFlags, QueryFlags, Filter, QueryFilterData, QueryCallback, DebugParams); }
#define DEFINE_LOW_LEVEL_SWEEP_ACCEL(TACCEL) \
DEFINE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, FHitSweep); \
DEFINE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, ChaosInterface::FPTSweepHit);
DEFINE_LOW_LEVEL_SWEEP_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DEFINE_LOW_LEVEL_SWEEP_ACCEL(IExternalSpatialAcceleration)
DEFINE_LOW_LEVEL_SWEEP_ACCEL(FPhysScene)

template <typename TAccelContainer, typename THitOverlap>
void LowLevelOverlapImpl(const TAccelContainer& Container, const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<THitOverlap>& HitBuffer, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams)
{
	const ChaosInterface::FSceneQueryCommonParams CommonParams(*QueryCallback, QueryFilterData, DebugParams);
	LowLevelOverlapImpl(Container, QueryGeom, GeomPose, HitBuffer, CommonParams);
}

#define DEFINE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, THIT) template<> void Chaos::Private::LowLevelOverlap(const TACCEL& Container, const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<THIT>& HitBuffer, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) \
{ LowLevelOverlapImpl(Container, QueryGeom, GeomPose, HitBuffer, QueryFlags, Filter, QueryFilterData, QueryCallback, DebugParams); }
#define DEFINE_LOW_LEVEL_OVERLAP_ACCEL(TACCEL) \
DEFINE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, FHitOverlap) \
DEFINE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, ChaosInterface::FPTOverlapHit)
DEFINE_LOW_LEVEL_OVERLAP_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DEFINE_LOW_LEVEL_OVERLAP_ACCEL(IExternalSpatialAcceleration)
DEFINE_LOW_LEVEL_OVERLAP_ACCEL(FPhysScene)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
