// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WorldCollision.cpp: UWorld collision implementation
=============================================================================*/

#include "WorldCollision.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/HitResult.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "Framework/Docking/TabManager.h"
#include "Collision.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "Chaos/ImplicitObject.h"
#include "Collision/CollisionConversions.h"
#include "SceneView.h"

using namespace PhysicsInterfaceTypes;

DEFINE_LOG_CATEGORY(LogCollision);

/** Collision stats */


DEFINE_STAT(STAT_Collision_SceneQueryTotal);
DEFINE_STAT(STAT_Collision_RaycastAny);
DEFINE_STAT(STAT_Collision_RaycastSingle);
DEFINE_STAT(STAT_Collision_RaycastMultiple);
DEFINE_STAT(STAT_Collision_GeomSweepAny);
DEFINE_STAT(STAT_Collision_GeomSweepSingle);
DEFINE_STAT(STAT_Collision_GeomSweepMultiple);
DEFINE_STAT(STAT_Collision_GeomOverlapMultiple);
DEFINE_STAT(STAT_Collision_GeomOverlapBlocking);
DEFINE_STAT(STAT_Collision_GeomOverlapAny);
DEFINE_STAT(STAT_Collision_FBodyInstance_OverlapMulti);
DEFINE_STAT(STAT_Collision_FBodyInstance_OverlapTest);
DEFINE_STAT(STAT_Collision_FBodyInstance_LineTrace);
DEFINE_STAT(STAT_Collision_PreFilter);
DEFINE_STAT(STAT_Collision_SpherecastMultiple_Internal);
DEFINE_STAT(STAT_Collision_GeomSweepMultiple_Internal);
DEFINE_STAT(STAT_Collision_RaycastMultiple_Internal);

/** default collision response container - to be used without reconstructing every time**/
FCollisionResponseContainer FCollisionResponseContainer::DefaultResponseContainer(ECR_Block);

/* This is default response param that's used by trace query **/
FCollisionResponseParams		FCollisionResponseParams::DefaultResponseParam;
FCollisionObjectQueryParams		FCollisionObjectQueryParams::DefaultObjectQueryParam;
FCollisionQueryParams			FCollisionQueryParams::DefaultQueryParam(SCENE_QUERY_STAT(DefaultQueryParam), true);
FComponentQueryParams			FComponentQueryParams::DefaultComponentQueryParams(SCENE_QUERY_STAT(DefaultComponentQueryParams));
FCollisionShape					FCollisionShape::LineShape;

// default being the 0. That isn't invalid, but ObjectQuery param overrides this 
ECollisionChannel DefaultCollisionChannel = (ECollisionChannel) 0;

//////////////////////////////////////////////////////////////////////////

/* Set functions for each Shape type */
void FBaseTraceDatum::Set(UWorld * World, const FCollisionShape& InCollisionShape, const FCollisionQueryParams& Param, const struct FCollisionResponseParams &InResponseParam, const struct FCollisionObjectQueryParams& InObjectQueryParam,
	ECollisionChannel Channel, uint32 InUserData, int32 FrameCounter)
{
	ensure(World);
	CollisionParams.CollisionShape = InCollisionShape;
	CollisionParams.CollisionQueryParam = Param;
	CollisionParams.ResponseParam = InResponseParam;
	CollisionParams.ObjectQueryParam = InObjectQueryParam;
	TraceChannel = Channel;
	UserData = InUserData;
	FrameNumber = FrameCounter;
	PhysWorld = World;
}

//////////////////////////////////////////////////////////////////////////

FTraceDatum::FTraceDatum() {}

/** Set Trace Datum for each shape type **/
FTraceDatum::FTraceDatum(UWorld* World, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Param, const struct FCollisionResponseParams& InResponseParam, const struct FCollisionObjectQueryParams& InObjectQueryParam,
	ECollisionChannel Channel, uint32 InUserData, EAsyncTraceType InTraceType, const FVector& InStart, const FVector& InEnd, const FQuat& InRot, const FTraceDelegate* InDelegate, int32 FrameCounter)
{
	Set(World, CollisionShape, Param, InResponseParam, InObjectQueryParam, Channel, InUserData, FrameCounter);
	Start = InStart;
	End = InEnd;
	Rot = InRot;
	if (InDelegate)
	{
		Delegate = *InDelegate;
	}
	else
	{
		Delegate.Unbind();
	}
	OutHits.Reset();
	TraceType = InTraceType;
}

//////////////////////////////////////////////////////////////////////////

FOverlapDatum::FOverlapDatum()
{}

FOverlapDatum::FOverlapDatum(UWorld* World, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Param, const FCollisionResponseParams& InResponseParam,
	const FCollisionObjectQueryParams& InObjectQueryParam, ECollisionChannel Channel, uint32 InUserData, const FVector& InPos, const FQuat& InRot,
	const FOverlapDelegate* InDelegate, int32 FrameCounter)
{
	Set(World, CollisionShape, Param, InResponseParam, InObjectQueryParam, Channel, InUserData, FrameCounter);
	Pos = InPos;
	Rot = InRot;
	if (InDelegate)
	{
		Delegate = *InDelegate;
	}
	else
	{
		Delegate.Unbind();
	}
	OutOverlaps.Reset();
}

//////////////////////////////////////////////////////////////////////////

bool UWorld::LineTraceTestByChannel(const FVector& Start,const FVector& End,ECollisionChannel TraceChannel, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	return FPhysicsInterface::RaycastTest(this, Start, End, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
}

bool UWorld::LineTraceSingleByChannel(struct FHitResult& OutHit,const FVector& Start,const FVector& End,ECollisionChannel TraceChannel,const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	return FPhysicsInterface::RaycastSingle(this, OutHit, Start, End, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
}

bool UWorld::LineTraceFromScreenProjection(FHitResult& OutHit, const FSceneViewProjectionData& ProjectionData, const FVector2D& ScreenPosition, const ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const float TraceDistance) const
{
	const FMatrix InvViewProjMatrix = ProjectionData.ComputeViewProjectionMatrix().InverseFast();

	FVector WorldOrigin;
	FVector WorldDirection;
	FSceneView::DeprojectScreenToWorld(ScreenPosition, ProjectionData.GetConstrainedViewRect(), InvViewProjMatrix, WorldOrigin, WorldDirection);

	return LineTraceSingleByChannel(OutHit, WorldOrigin, WorldOrigin + WorldDirection * TraceDistance, TraceChannel, Params);
}

bool UWorld::LineTraceMultiByChannel(TArray<struct FHitResult>& OutHits,const FVector& Start,const FVector& End,ECollisionChannel TraceChannel,const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	return FPhysicsInterface::RaycastMulti(this, OutHits, Start, End, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
}

bool UWorld::SweepTestByChannel(const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	if (CollisionShape.IsNearlyZero())
	{
		// if extent is 0, we'll just do linetrace instead
		return LineTraceTestByChannel(Start, End, TraceChannel, Params, ResponseParam);
	}
	else
	{
		return FPhysicsInterface::GeomSweepTest(this, CollisionShape, Rot, Start, End, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
	}
}

bool UWorld::SweepSingleByChannel(struct FHitResult& OutHit, const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	if (CollisionShape.IsNearlyZero())
	{
		return LineTraceSingleByChannel(OutHit, Start, End, TraceChannel, Params, ResponseParam);
	}
	else
	{
		return FPhysicsInterface::GeomSweepSingle(this, CollisionShape, Rot, OutHit, Start, End, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
	}
}

bool UWorld::SweepMultiByChannel(TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	if (CollisionShape.IsNearlyZero())
	{
		return LineTraceMultiByChannel(OutHits, Start, End, TraceChannel, Params, ResponseParam);
	}
	else
	{
		return FPhysicsInterface::GeomSweepMulti(this, CollisionShape, Rot, OutHits, Start, End, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
	}
}

bool UWorld::OverlapBlockingTestByChannel(const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	bool bBlocking = false;
	bBlocking = FPhysicsInterface::GeomOverlapBlockingTest(this, CollisionShape, Pos, Rot, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
	return bBlocking;

}

bool UWorld::OverlapAnyTestByChannel(const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	bool bBlocking = false;
	bBlocking = FPhysicsInterface::GeomOverlapAnyTest(this, CollisionShape, Pos, Rot, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
	return bBlocking;

}

bool UWorld::OverlapMultiByChannel(TArray<struct FOverlapResult>& OutOverlaps, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */) const
{
	return FPhysicsInterface::GeomOverlapMulti(this, CollisionShape, Pos, Rot, OutOverlaps, TraceChannel, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
}

// object query interfaces

bool UWorld::OverlapMultiByObjectType(TArray<struct FOverlapResult>& OutOverlaps, const FVector& Pos, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */) const
{
	FPhysicsInterface::GeomOverlapMulti(this, CollisionShape, Pos, Rot, OutOverlaps, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams);

	// object query returns true if any hit is found, not only blocking hit
	return (OutOverlaps.Num() > 0);
}

bool UWorld::LineTraceTestByObjectType(const FVector& Start,const FVector& End,const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */) const
{
	return FPhysicsInterface::RaycastTest(this, Start, End, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams);
}

bool UWorld::LineTraceSingleByObjectType(struct FHitResult& OutHit,const FVector& Start,const FVector& End,const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */) const
{
	return FPhysicsInterface::RaycastSingle(this, OutHit, Start, End, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams);
}

bool UWorld::LineTraceMultiByObjectType(TArray<struct FHitResult>& OutHits,const FVector& Start,const FVector& End,const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */) const
{
	FPhysicsInterface::RaycastMulti(this, OutHits, Start, End, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams);

	// object query returns true if any hit is found, not only blocking hit
	return (OutHits.Num() > 0);
}

bool UWorld::SweepTestByObjectType(const FVector& Start, const FVector& End, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */) const
{
	if (CollisionShape.IsNearlyZero())
	{
		// if extent is 0, we'll just do linetrace instead
		return LineTraceTestByObjectType(Start, End, ObjectQueryParams, Params);
	}
	else
	{
		return FPhysicsInterface::GeomSweepTest(this, CollisionShape, Rot, Start, End, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams);
	}

}

bool UWorld::SweepSingleByObjectType(struct FHitResult& OutHit, const FVector& Start, const FVector& End, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */) const
{
	if (CollisionShape.IsNearlyZero())
	{
		return LineTraceSingleByObjectType(OutHit, Start, End, ObjectQueryParams, Params);
	}
	else
	{
		return FPhysicsInterface::GeomSweepSingle(this, CollisionShape, Rot, OutHit, Start, End, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams);
	}

}

bool UWorld::SweepMultiByObjectType(TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */) const
{
	if (CollisionShape.IsNearlyZero())
	{
		return LineTraceMultiByObjectType(OutHits, Start, End, ObjectQueryParams, Params);
	}
	else
	{
		FPhysicsInterface::GeomSweepMulti(this, CollisionShape, Rot, OutHits, Start, End, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams);

		// object query returns true if any hit is found, not only blocking hit
		return (OutHits.Num() > 0);
	}
}

bool UWorld::OverlapAnyTestByObjectType(const FVector& Pos, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */) const
{
	bool bBlocking = false;
	bBlocking = FPhysicsInterface::GeomOverlapAnyTest(this, CollisionShape, Pos, Rot, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams);
	return bBlocking;
}

// profile interfaces

bool UWorld::LineTraceTestByProfile(const FVector& Start, const FVector& End, FName ProfileName, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return LineTraceTestByChannel(Start, End, TraceChannel, Params, ResponseParam);
}

bool UWorld::LineTraceSingleByProfile(struct FHitResult& OutHit, const FVector& Start, const FVector& End, FName ProfileName, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return LineTraceSingleByChannel(OutHit, Start, End, TraceChannel, Params, ResponseParam);
}

bool UWorld::LineTraceMultiByProfile(TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, FName ProfileName, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return LineTraceMultiByChannel(OutHits, Start, End, TraceChannel, Params, ResponseParam);
}

bool UWorld::SweepTestByProfile(const FVector& Start, const FVector& End, const FQuat& Rot, FName ProfileName, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return SweepTestByChannel(Start, End, Rot, TraceChannel, CollisionShape, Params, ResponseParam);
}

bool UWorld::SweepSingleByProfile(struct FHitResult& OutHit, const FVector& Start, const FVector& End, const FQuat& Rot, FName ProfileName, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return SweepSingleByChannel(OutHit, Start, End, Rot, TraceChannel, CollisionShape, Params, ResponseParam);
}

bool UWorld::SweepMultiByProfile(TArray<FHitResult>& OutHits, const FVector& Start, const FVector& End, const FQuat& Rot, FName ProfileName, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return SweepMultiByChannel(OutHits, Start, End, Rot, TraceChannel, CollisionShape, Params, ResponseParam);
}

bool UWorld::OverlapBlockingTestByProfile(const FVector& Pos, const FQuat& Rot, FName ProfileName, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return OverlapBlockingTestByChannel(Pos, Rot, TraceChannel, CollisionShape, Params, ResponseParam);
}

bool UWorld::OverlapAnyTestByProfile(const FVector& Pos, const FQuat& Rot, FName ProfileName, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return OverlapAnyTestByChannel(Pos, Rot, TraceChannel, CollisionShape, Params, ResponseParam);
}

bool UWorld::OverlapMultiByProfile(TArray<struct FOverlapResult>& OutOverlaps, const FVector& Pos, const FQuat& Rot, FName ProfileName, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params) const
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return OverlapMultiByChannel(OutOverlaps, Pos, Rot, TraceChannel, CollisionShape, Params, ResponseParam);
}


bool UWorld::ComponentOverlapMulti(TArray<struct FOverlapResult>& OutOverlaps, const class UPrimitiveComponent* PrimComp, const FVector& Pos, const FQuat& Quat, const FComponentQueryParams& Params, const FCollisionObjectQueryParams& ObjectQueryParams) const
{
	if (PrimComp)
	{
		ComponentOverlapMultiByChannel(OutOverlaps, PrimComp, Pos, Quat, PrimComp->GetCollisionObjectType(), Params, ObjectQueryParams);
		
		// object query returns true if any hit is found, not only blocking hit
		return (OutOverlaps.Num() > 0);
	}
	else
	{
		UE_LOGF(LogCollision, Log, "ComponentOverlapMulti : No PrimComp");
		return false;
	}
}

bool UWorld::ComponentOverlapMultiByChannel(TArray<struct FOverlapResult>& OutOverlaps, const class UPrimitiveComponent* PrimComp, const FVector& Pos, const FQuat& Quat, ECollisionChannel TraceChannel, const FComponentQueryParams& Params, const FCollisionObjectQueryParams& ObjectQueryParams) const
{
	if (!GetPhysicsScene())
	{
		return false;
	}

	if (!PrimComp)
	{
		UE_LOGF(LogCollision, Log, "ComponentOverlapMultiByChannel : No PrimComp");
		return false;
	}

	FComponentQueryParams ParamsWithSelf = Params;
	ParamsWithSelf.AddIgnoredComponent_LikelyDuplicatedRoot(PrimComp);
	OutOverlaps.Reset();

	// Maintains compatibility with previous versions that primarily depended on the body instance.
	FBodyInstance* BodyInstance = PrimComp->GetBodyInstance();
	if (BodyInstance)
	{
		return BodyInstance->OverlapMulti(OutOverlaps, this, nullptr, Pos, Quat, TraceChannel, ParamsWithSelf, FCollisionResponseParams{ BodyInstance->GetResponseToChannels() }, ObjectQueryParams);
	}

	// New version that's more generic and relies on the physics object interface.
	TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = PrimComp->GetAllPhysicsObjects();
	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(PhysicsObjects);

	// For geometry collections we want to make sure we don't account for disabled particles since that means their parent cluster still hasn't released them yet.
	// This should be safe for non-geometry collections and not filter out anything.
	PhysicsObjects = PhysicsObjects.FilterByPredicate(
		[&Interface](Chaos::FPhysicsObject* Object)
		{
			return !Interface->AreAllDisabled({ &Object, 1 });
		}
	);

	if (PhysicsObjects.IsEmpty())
	{
		UE_LOGF(LogCollision, Log, "UWorld::ComponentOverlapMultiByChannel : (%ls) No physics data", *PrimComp->GetName());
		return false;
	}

	bool bHaveBlockingHit = false;
	FCollisionResponseParams CollisionResponseParams{ PrimComp->GetCollisionResponseToChannels() };

	const FTransform ComponentToTest{ Quat, Pos };
	const FTransform WorldToComponent = PrimComp->GetComponentToWorld().Inverse();

	TArray<struct FOverlapResult> TempOverlaps;
	for (Chaos::FPhysicsObjectHandle Object : PhysicsObjects)
	{
		TArray<Chaos::FShapeInstanceProxy*> Shapes = Interface->GetAllThreadShapes({&Object, 1});

		// Determine how to convert the local space of this body instance to the test space
		const FTransform ObjectToWorld = Interface->GetTransform(Object);
		const FTransform ObjectToTest = ComponentToTest * WorldToComponent * ObjectToWorld;

		for (Chaos::FShapeInstanceProxy* Shape : Shapes)
		{
			FPhysicsShapeHandle ShapeRef{ Shape, nullptr };

			// TODO: Add support to be able to check if the shape collision is enabled for the generic physics object interface.
			/*
			// Skip this shape if it's CollisionEnabled setting was masked out
			if (ParamsWithSelf.ShapeCollisionMask && !(ParamsWithSelf.ShapeCollisionMask & BodyInstance->GetShapeCollisionEnabled(ShapeIdx)))
			{
				continue;
			}
			*/
			FPhysicsGeometryCollection GeomCollection = FPhysicsInterface::GetGeometryCollection(ShapeRef);
			if (!ShapeRef.GetGeometry().IsConvex())
			{
				continue;	//we skip complex shapes - should this respect ComplexAsSimple?
			}

			TempOverlaps.Reset();
			if (FPhysicsInterface::GeomOverlapMulti(this, GeomCollection, ObjectToTest.GetTranslation(), ObjectToTest.GetRotation(), TempOverlaps, TraceChannel, ParamsWithSelf, CollisionResponseParams, ObjectQueryParams))
			{
				bHaveBlockingHit = true;
			}
			OutOverlaps.Append(TempOverlaps);
		}
	}

	return bHaveBlockingHit;
}

bool UWorld::ComponentSweepMulti(TArray<struct FHitResult>& OutHits, class UPrimitiveComponent* PrimComp, const FVector& Start, const FVector& End, const FQuat& Quat, const FComponentQueryParams& Params) const
{
	if (PrimComp)
	{
		return ComponentSweepMultiByChannel(OutHits, PrimComp, Start, End, Quat, PrimComp->GetCollisionObjectType(), Params);
	}
	else
	{
		UE_LOGF(LogCollision, Log, "ComponentSweepMulti : No PrimComp");
		return false;
	}
}

bool UWorld::ComponentSweepMultiByChannel(TArray<struct FHitResult>& OutHits, class UPrimitiveComponent* PrimComp, const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FComponentQueryParams& Params) const
{
	OutHits.Reset();

	if (GetPhysicsScene() == NULL)
	{
		return false;
	}

	if (PrimComp == NULL)
	{
		UE_LOGF(LogCollision, Log, "ComponentSweepMultiByChannel : No PrimComp");
		return false;
	}

	// if extent is 0, do line trace
	if (PrimComp->IsZeroExtent())
	{
		return FPhysicsInterface::RaycastMulti(this, OutHits, Start, End, TraceChannel, Params, FCollisionResponseParams(PrimComp->GetCollisionResponseToChannels()));
	}

	TArray<Chaos::FPhysicsObject*> PhysicsObjects = PrimComp->GetAllPhysicsObjects();
	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(PhysicsObjects);

	// For geometry collections we want to make sure we don't account for disabled particles since that means their parent cluster still hasn't released them yet.
	// This should be safe for non-geometry collections and not filter out anything.
	PhysicsObjects = PhysicsObjects.FilterByPredicate(
		[&Interface](Chaos::FPhysicsObject* Object)
		{
			return !Interface->AreAllDisabled({ &Object, 1 });
		}
	);

	SCOPE_CYCLE_COUNTER(STAT_Collision_GeomSweepMultiple);
	bool bHaveBlockingHit = false;

	TArray<FHitResult> TempHits;
	for (Chaos::FPhysicsObjectHandle Object : PhysicsObjects)
	{
		FComponentQueryParams ParamsCopy{ Params };
		TSet<uint32> ActorsToExclude;
		for (uint32 ActorID : ParamsCopy.GetIgnoredSourceObjects())
		{
			ActorsToExclude.Add(ActorID);
		}		
		ParamsCopy.ClearIgnoredSourceObjects(); // This will be populated a bit later
		// All actors pointed to by shapes should be ignored (This deals with welded Actors)
		TArray<Chaos::FShapeInstanceProxy*> Shapes = Interface->GetAllThreadShapes({ &Object, 1 });
		for (Chaos::FShapeInstanceProxy* Shape : Shapes)
		{
			const Chaos::Filter::FInstanceData InstanceData = ChaosInterface::GetFilterInstanceData(*Shape);
			uint32 ShapeActorID = InstanceData.GetOwnerId();
			ActorsToExclude.Add(ShapeActorID);
		}

		for (uint32 ActorID : ActorsToExclude)
		{
			ParamsCopy.AddIgnoredActor(ActorID);
		}

		for (Chaos::FShapeInstanceProxy* Shape : Shapes)
		{
			FPhysicsShapeHandle ShapeHandle{ Shape, nullptr };

			check(ShapeHandle.IsValid());
			ECollisionShapeType ShapeType = FPhysicsInterface::GetShapeType(ShapeHandle);

			if (!ShapeHandle.GetGeometry().IsConvex())
			{
				//We skip complex shapes. Should this respect complex as simple?
				continue;
			}

			FPhysicsGeometryCollection GeomCollection = FPhysicsInterface::GetGeometryCollection(ShapeHandle);
			TempHits.Reset();
			if (FPhysicsInterface::GeomSweepMulti(this, GeomCollection, Rot, TempHits, Start, End, TraceChannel, ParamsCopy, FCollisionResponseParams(PrimComp->GetCollisionResponseToChannels())))
			{
				bHaveBlockingHit = true;
			}
			OutHits.Append(TempHits);	//todo: should these be made unique?
		}
	}

	OutHits.Sort(FCompareFHitResultTime());
	return bHaveBlockingHit;
}

#if ENABLE_COLLISION_ANALYZER


static class FCollisionExec : private FSelfRegisteringExec
{
protected:
	/** Console commands, see embeded usage statement **/
	virtual bool Exec_Dev( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
#if ENABLE_COLLISION_ANALYZER
		if (FParse::Command(&Cmd, TEXT("CANALYZER")))
		{
			FGlobalTabmanager::Get()->TryInvokeTab(FName("CollisionAnalyzerApp"));
			return true;
		}
#endif // ENABLE_COLLISION_ANALYZER
		return false;
	}
} CollisionExec;

#endif // ENABLE_COLLISION_ANALYZER

AsyncTraceData::AsyncTraceData()
	: NumQueuedTraceData(0)
	, NumQueuedOverlapData(0)
	, bAsyncAllowed(false)
	, bAsyncTasksCompleted(false)
{}

AsyncTraceData::~AsyncTraceData() = default;
