// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WorldCollisionAsync.cpp: UWorld async collision implementation
=============================================================================*/

#include "Engine/World.h"
#include "Async/TaskGraphInterfaces.h"
#include "AutoRTFM.h"
#include "Engine/HitResult.h"
#include "Engine/OverlapResult.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "Misc/Fork.h"
#include "Physics/Experimental/PhysInterface_Chaos.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "UObject/RemoteExecutor.h"

CSV_DEFINE_CATEGORY(WorldCollision, true);
DEFINE_LOG_CATEGORY_STATIC(LogWorldCollision, Log, All);

/**
 * Async trace functions
 * Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
 *
 * @param InDelegate	Delegate function to be called - to see example, search FTraceDelegate
 *						Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
 *						Before sending to the function, 
 *						
 *						FTraceDelegate TraceDelegate;
 *						TraceDelegate.BindRaw(this, &MyActor::TraceDone);
 * 
 * @param UserData		UserData
 */

namespace AsyncTraceCVars
{
	static int32 RunAsyncTraceOnWorkerThread = 1;
	static FAutoConsoleVariableRef CVarRunAsyncTraceOnWorkerThread(
		TEXT("RunAsyncTraceOnWorkerThread"),
		RunAsyncTraceOnWorkerThread,
		TEXT("Whether to use worker thread for async trace functionality. This works if FApp::ShouldUseThreadingForPerformance is true. Otherwise it will always use game thread. \n")
		TEXT("0: Use game thread, 1: User worker thread"),
		ECVF_Default);

	bool IsAsyncTraceOnWorkerThreads()
	{
		return RunAsyncTraceOnWorkerThread != 0 && (FApp::ShouldUseThreadingForPerformance() || FForkProcessHelper::IsForkedMultithreadInstance());
	}

	static float AsyncTraceDelegateHitchThresholdMS = -1.0f;
	static FAutoConsoleVariableRef CVarAsyncTraceDelegateHitchThresholdMS(
		TEXT("AsyncTraceDelegateHitchThresholdMS"),
		AsyncTraceDelegateHitchThresholdMS,
		TEXT("During ResetAsyncTrace, if the delegate dispatch step takes longer than this time (in Milliseconds) then output a list of the delegates to the log\n")
		TEXT("Value of < 0 will deactivate this functionality"),
		ECVF_Default);

	static float AsyncTraceDelegateLoggingIntervalSeconds = 30.0f;
	static FAutoConsoleVariableRef CVarAsyncTraceDelegateLoggingIntervalSeconds(
		TEXT("AsyncTraceDelegateLoggingIntervalSeconds"),
		AsyncTraceDelegateLoggingIntervalSeconds,
		TEXT("If AsyncTraceDelegateHitchThresholdMS is > 0, then this value defines the minimum time in between the hitch logging, so we avoid LogSpam\n"),
		ECVF_Default);
}

namespace
{
	// Helper functions to return the right named member container based on a datum type
	template <typename DatumType> FORCEINLINE TArray<TUniquePtr<TTraceThreadData<DatumType    >>>& GetTraceContainer               (AsyncTraceData& DataBuffer);
	template <>                   FORCEINLINE TArray<TUniquePtr<TTraceThreadData<FTraceDatum  >>>& GetTraceContainer<FTraceDatum>  (AsyncTraceData& DataBuffer) { return DataBuffer.TraceData;   }
	template <>                   FORCEINLINE TArray<TUniquePtr<TTraceThreadData<FOverlapDatum>>>& GetTraceContainer<FOverlapDatum>(AsyncTraceData& DataBuffer) { return DataBuffer.OverlapData; }

	// Helper functions to return the right named member trace index based on a datum type
	template <typename DatumType> FORCEINLINE int32& GetTraceIndex(AsyncTraceData& DataBuffer);
	template <>                   FORCEINLINE int32& GetTraceIndex<FTraceDatum>(AsyncTraceData& DataBuffer) { return DataBuffer.NumQueuedTraceData; }
	template <>                   FORCEINLINE int32& GetTraceIndex<FOverlapDatum>(AsyncTraceData& DataBuffer) { return DataBuffer.NumQueuedOverlapData; }

	template <typename DatumType> FORCEINLINE FTransactionalAsyncTraceBuffer<DatumType>& GetTransactionalData(AsyncTraceData& DataBuffer);
	template <> FORCEINLINE FTransactionalAsyncTraceBuffer<FTraceDatum>& GetTransactionalData(AsyncTraceData& DataBuffer) { return DataBuffer.TransactionalTraceData; }
	template <> FORCEINLINE FTransactionalAsyncTraceBuffer<FOverlapDatum>& GetTransactionalData(AsyncTraceData& DataBuffer) { return DataBuffer.TransactionalOverlapData; }

	/** For referencing a thread data buffer and a datum within it */
	struct FBufferIndexPair
	{
		int32 Block;
		int32 Index;

		explicit FBufferIndexPair(int32 InVal)
			: Block(InVal / ASYNC_TRACE_BUFFER_SIZE)
			, Index(InVal % ASYNC_TRACE_BUFFER_SIZE)
		{
		}

		FBufferIndexPair(int32 InBlock, int32 InIndex)
			: Block(InBlock)
			, Index(InIndex)
		{
		}

		template <typename DatumType>
		DatumType* DatumLookup(TArray<TUniquePtr<TTraceThreadData<DatumType>>>& Array) const
		{
			// if not valid index, return
			if (!Array.IsValidIndex(Block))
			{
				return NULL;
			}

			if (Index < 0 || Index >= ASYNC_TRACE_BUFFER_SIZE)
			{
				return NULL;
			}

			return &Array[Block]->Buffer[Index];
		}

		template <typename DatumType>
		FORCEINLINE DatumType& DatumLookupChecked(TArray<TUniquePtr<TTraceThreadData<DatumType>>>& Array) const
		{
			check(Index >= 0 && Index < ASYNC_TRACE_BUFFER_SIZE);
			return Array[Block]->Buffer[Index];
		}
	};

	void RunTraceTask(FTraceDatum* TraceDataBuffer, int32 TotalCount)
	{
		check(TraceDataBuffer);

		for (; TotalCount; --TotalCount)
		{
			FTraceDatum& TraceData = *TraceDataBuffer++;
			TraceData.OutHits.Empty();

			if (TraceData.PhysWorld.IsValid())
			{
				if ((TraceData.CollisionParams.CollisionShape.ShapeType == ECollisionShape::Line) || TraceData.CollisionParams.CollisionShape.IsNearlyZero())
				{
					// MULTI
					if (TraceData.TraceType == EAsyncTraceType::Multi)
					{
						FPhysicsInterface::RaycastMulti(TraceData.PhysWorld.Get(), TraceData.OutHits, TraceData.Start, TraceData.End, TraceData.TraceChannel,
							TraceData.CollisionParams.CollisionQueryParam, TraceData.CollisionParams.ResponseParam, TraceData.CollisionParams.ObjectQueryParam);
					}
					// SINGLE
					else if(TraceData.TraceType == EAsyncTraceType::Single)
					{
						FHitResult Result;

						bool bHit = FPhysicsInterface::RaycastSingle(TraceData.PhysWorld.Get(), Result, TraceData.Start, TraceData.End, TraceData.TraceChannel,
							TraceData.CollisionParams.CollisionQueryParam, TraceData.CollisionParams.ResponseParam, TraceData.CollisionParams.ObjectQueryParam);

						if(bHit)
						{
							TraceData.OutHits.Add(Result);
						}
					}
					// TEST
					else
					{
						bool bHit = FPhysicsInterface::RaycastTest(TraceData.PhysWorld.Get(), TraceData.Start, TraceData.End, TraceData.TraceChannel,
							TraceData.CollisionParams.CollisionQueryParam, TraceData.CollisionParams.ResponseParam, TraceData.CollisionParams.ObjectQueryParam);

						if(bHit)
						{
							FHitResult Result;
							Result.bBlockingHit = true;
							TraceData.OutHits.Add(Result);
						}
					}
				}
				else
				{
					// MULTI
					if (TraceData.TraceType == EAsyncTraceType::Multi)
					{
						FPhysicsInterface::GeomSweepMulti(TraceData.PhysWorld.Get(), TraceData.CollisionParams.CollisionShape, TraceData.Rot, TraceData.OutHits, TraceData.Start, TraceData.End, TraceData.TraceChannel,
							TraceData.CollisionParams.CollisionQueryParam, TraceData.CollisionParams.ResponseParam, TraceData.CollisionParams.ObjectQueryParam);
					}
					// SINGLE
					else if (TraceData.TraceType == EAsyncTraceType::Single)
					{
						FHitResult Result;

						bool bHit = FPhysicsInterface::GeomSweepSingle(TraceData.PhysWorld.Get(), TraceData.CollisionParams.CollisionShape, TraceData.Rot, Result, TraceData.Start, TraceData.End, TraceData.TraceChannel,
							TraceData.CollisionParams.CollisionQueryParam, TraceData.CollisionParams.ResponseParam, TraceData.CollisionParams.ObjectQueryParam);

						if(bHit)
						{
							TraceData.OutHits.Add(Result);
						}
					}
					// TEST
					else
					{
						bool bHit = FPhysicsInterface::GeomSweepTest(TraceData.PhysWorld.Get(), TraceData.CollisionParams.CollisionShape, TraceData.Rot, TraceData.Start, TraceData.End, TraceData.TraceChannel,
							TraceData.CollisionParams.CollisionQueryParam, TraceData.CollisionParams.ResponseParam, TraceData.CollisionParams.ObjectQueryParam);

						if(bHit)
						{
							FHitResult Result;
							Result.bBlockingHit = true;
							TraceData.OutHits.Add(Result);
						}						
					}
				}
			}
		}
	}

	EPhysicsQueryKind TraceTypeToQueryKind(EAsyncTraceType TraceType)
	{
		if (TraceType == EAsyncTraceType::Multi)
		{
			return EPhysicsQueryKind::Multi;
		}
		else if (TraceType == EAsyncTraceType::Single)
		{
			return EPhysicsQueryKind::Single;
		}
		else
		{
			return EPhysicsQueryKind::Test;
		}
	}

	void RunTransactionalTraceTask(UPhysicsQueryHandler& QueryHandler, UWorld* World, FTraceDatum& TraceData, const FTraceHandle& TraceHandle)
	{
		TraceData.OutHits.Empty();

		if (TraceData.PhysWorld.IsValid())
		{
			EPhysicsQueryKind QueryKind = TraceTypeToQueryKind(TraceData.TraceType);
			Chaos::EThreadQueryContext ThreadContext = Chaos::EThreadQueryContext::GTData;
			Chaos::FCommonQueryData CommonData;
			CommonData.TraceChannel = TraceData.TraceChannel;
			CommonData.Params = TraceData.CollisionParams.CollisionQueryParam;
			CommonData.ResponseParams = TraceData.CollisionParams.ResponseParam;
			CommonData.ObjectParams = TraceData.CollisionParams.ObjectQueryParam;

			if ((TraceData.CollisionParams.CollisionShape.ShapeType == ECollisionShape::Line) || TraceData.CollisionParams.CollisionShape.IsNearlyZero())
			{
				Chaos::FRayQueryData RayData;
				RayData.Start = TraceData.Start;
				RayData.End = TraceData.End;
				
				QueryHandler.QueueAsyncRaycast(TraceHandle, QueryKind, ThreadContext, World, RayData, CommonData);
			}
			else
			{
				Chaos::FSweepQueryData SweepData;
				SweepData.Start = TraceData.Start;
				SweepData.End = TraceData.End;
				SweepData.GeomRot = TraceData.Rot;
				SweepData.QueryShape.CollisionShape = TraceData.CollisionParams.CollisionShape;

				QueryHandler.QueueAsyncSweep(TraceHandle, QueryKind, ThreadContext, World, SweepData, CommonData);
			}
		}
	}

	void RunTraceTask(FOverlapDatum* OverlapDataBuffer, int32 TotalCount)
	{
		check(OverlapDataBuffer);

		for (; TotalCount; --TotalCount)
		{
			FOverlapDatum& OverlapData = *OverlapDataBuffer++;
			OverlapData.OutOverlaps.Empty();

			if (!OverlapData.PhysWorld.IsValid())
			{
				continue;
			}

			FPhysicsInterface::GeomOverlapMulti(
				OverlapData.PhysWorld.Get(),
				OverlapData.CollisionParams.CollisionShape,
				OverlapData.Pos,
				OverlapData.Rot,
				OverlapData.OutOverlaps,
				OverlapData.TraceChannel,
				OverlapData.CollisionParams.CollisionQueryParam,
				OverlapData.CollisionParams.ResponseParam,
				OverlapData.CollisionParams.ObjectQueryParam);
		}
	}

	void RunTransactionalTraceTask(UPhysicsQueryHandler& QueryHandler, UWorld* World, FOverlapDatum& OverlapDatum, const FTraceHandle& TraceHandle)
	{
		OverlapDatum.OutOverlaps.Empty();

		if (!OverlapDatum.PhysWorld.IsValid())
		{
			return;
		}

		Chaos::EThreadQueryContext ThreadContext = Chaos::EThreadQueryContext::GTData;
		Chaos::EQueryInfo QueryInfo = Chaos::EQueryInfo::GatherAll;
		Chaos::FOverlapQueryData OverlapData;
		OverlapData.GeomPose = FTransform(OverlapDatum.Rot, OverlapDatum.Pos);
		OverlapData.QueryShape.CollisionShape = OverlapDatum.CollisionParams.CollisionShape;
		Chaos::FCommonQueryData CommonData;
		CommonData.TraceChannel = OverlapDatum.TraceChannel;
		CommonData.Params = OverlapDatum.CollisionParams.CollisionQueryParam;
		CommonData.ResponseParams = OverlapDatum.CollisionParams.ResponseParam;
		CommonData.ObjectParams = OverlapDatum.CollisionParams.ObjectQueryParam;

		QueryHandler.QueueAsyncOverlap(TraceHandle, QueryInfo, ThreadContext, World, OverlapData, CommonData);
	}

	FAutoConsoleTaskPriority CPrio_FAsyncTraceTask(
		TEXT("TaskGraph.TaskPriorities.AsyncTraceTask"),
		TEXT("Task and thread priority for async traces."),
		ENamedThreads::NormalThreadPriority, // Use Normal thread and normal task priority
		ENamedThreads::NormalTaskPriority 
		);


	/** Helper class define the task of Async Trace running**/
	class FAsyncTraceTask
	{
		// this accepts either trace or overlap data array
		// don't use both of them, it won't work
		FTraceDatum*   TraceData;
		FOverlapDatum* OverlapData;

		// data count
		int32 DataCount;

	public:
		FAsyncTraceTask(FTraceDatum* InTraceData, int32 InDataCount)
		{
			check(InTraceData);
			check(InDataCount > 0);

			TraceData   = InTraceData;
			OverlapData = NULL;
			DataCount   = InDataCount;
		}

		FAsyncTraceTask(FOverlapDatum* InOverlapData, int32 InDataCount)
		{
			check(InOverlapData);
			check(InDataCount > 0);

			TraceData   = NULL;
			OverlapData = InOverlapData;
			DataCount   = InDataCount;
		}

		static FORCEINLINE TStatId GetStatId()
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncTraceTask, STATGROUP_TaskGraphTasks);
		}
		/** return the thread for this task **/
		static FORCEINLINE ENamedThreads::Type GetDesiredThread()
		{
			return CPrio_FAsyncTraceTask.Get();
		}
		static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode()
		{ 
			return ESubsequentsMode::TrackSubsequents; 
		}
		/** 
		 *	Actually execute the tick.
		 *	@param	CurrentThread; the thread we are running on
		 *	@param	MyCompletionGraphEvent; my completion event. Not always useful since at the end of DoWork, you can assume you are done and hence further tasks do not need you as a prerequisite. 
		 *	However, MyCompletionGraphEvent can be useful for passing to other routines or when it is handy to set up subsequents before you actually do work.
		 **/
		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
		{
			if (TraceData)
			{
				RunTraceTask(TraceData, DataCount);
			}
			else if (OverlapData)
			{
				RunTraceTask(OverlapData, DataCount);
			}
		}
	};

	// This runs each chunk whenever filled up to GAsyncChunkSizeToIncrement OR when ExecuteAll is true
	template <typename DatumType>
	void ExecuteAsyncTraceIfAvailable(FWorldAsyncTraceState& State, bool bExecuteAll)
	{
		AsyncTraceData& DataBuffer = State.GetBufferForCurrentFrame();

		FBufferIndexPair Next(GetTraceIndex<DatumType>(DataBuffer));

		// when Next.Index == 0, and Next.Block > 0 , that means next one will be in the next buffer
		// but that case we'd like to send to thread
		if (Next.Index == 0 && Next.Block > 0)
		{
			Next.Block = Next.Block - 1;
			Next.Index = ASYNC_TRACE_BUFFER_SIZE;
		}
		// don't execute if we haven't been explicitly requested to OR there's nothing to run
		else if (!bExecuteAll || Next.Index == 0)
		{
			return;
		}

		DatumType* Datum      = GetTraceContainer<DatumType>(DataBuffer)[Next.Block]->Buffer;
		const bool bRunAsyncTraceOnWorkerThread = AsyncTraceCVars::IsAsyncTraceOnWorkerThreads();
		if (bRunAsyncTraceOnWorkerThread && !AutoRTFM::IsTransactional())
		{
			DataBuffer.AsyncTraceCompletionEvent.Emplace(TGraphTask<FAsyncTraceTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(Datum, Next.Index));
		}
		else
		{
			RunTraceTask(Datum, Next.Index);
		}
	}
	
	template <typename DatumType>
	void ExecuteAsyncTransactionalTraceIfAvailable(UWorld* World, FWorldAsyncTraceState& State, bool bExecuteAll)
	{
		AsyncTraceData& FrameBuffer = State.GetBufferForCurrentFrame();

		FTransactionalAsyncTraceBuffer<DatumType>& TransactionalData = GetTransactionalData<DatumType>(FrameBuffer);

		// Nothing to run
		if (TransactionalData.NumQueued == 0)
		{
			return;
		}

		// Can only run if we have a query handler
		UPhysicsQueryHandler* QueryHandler = World->PhysicsQueryHandler;
		if (QueryHandler == nullptr)
		{
			return;
		}

		for (int32 Index = 0; Index < TransactionalData.NumQueued; ++Index)
		{
			FBufferIndexPair Next(Index);
			DatumType* DatumBuffer = TransactionalData.Data[Next.Block]->Buffer;
			DatumType& Datum = DatumBuffer[Next.Index];

			FTraceHandle TraceHandle(State.CurrentFrame, Index, true);
			RunTransactionalTraceTask(*QueryHandler, World, Datum, TraceHandle);
		}
	}

	template <typename DatumType>
	FTraceHandle StartNewTrace(UWorld* World, FWorldAsyncTraceState& State, const DatumType& Val)
	{
		// Using async traces outside of the game thread can cause memory corruption
		check(IsInGameThread());

		// Get the buffer for the current frame
		AsyncTraceData& DataBuffer = State.GetBufferForCurrentFrame();

		// Check we're allowed to do an async call here
		check(DataBuffer.bAsyncAllowed);

		// Handle transactional async queries separately if we have a query handler. These need to be thrown into a separate queue 
		// that is processed outside of a transaction (we cannot process in "batches" like normal async).
		if (AutoRTFM::IsClosed() && World->PhysicsQueryHandler != nullptr)
		{
			FTransactionalAsyncTraceBuffer<DatumType>& TransactionalData = GetTransactionalData<DatumType>(DataBuffer);
			TransactionalData.Data.Add(MakeUnique<TTraceThreadData<DatumType>>());
			
			TArray<TUniquePtr<TTraceThreadData<DatumType>>>& TraceData = TransactionalData.Data;
			int32& TraceIndex = TransactionalData.NumQueued;

			// Indices are calculated as if the buffer was one contiguous array. Convert the chunked array size to the contiguous size.
			int32 LastAvailableIndex = TraceData.Num() * ASYNC_TRACE_BUFFER_SIZE;

			// if smaller than next available index
			if (LastAvailableIndex <= TraceIndex)
			{
				// add one more buffer
				TraceData.Add(MakeUnique<TTraceThreadData<DatumType>>());
				// We just resized to account for the next item. This shouldn't be able to fail.
				check(TraceData.Num() * ASYNC_TRACE_BUFFER_SIZE > TraceIndex);
			}

			FTraceHandle Result(State.CurrentFrame, TraceIndex, true);
			FBufferIndexPair(TraceIndex).DatumLookupChecked(TraceData) = Val;

			++TraceIndex;

			return Result;
		}
		else
		{
			TArray<TUniquePtr<TTraceThreadData<DatumType>>>& TraceData = GetTraceContainer<DatumType>(DataBuffer);
			int32& TraceIndex = GetTraceIndex<DatumType>(DataBuffer);

			// we calculate index as continuous, not as each chunk, but continuous way 
			int32 LastAvailableIndex = TraceData.Num() * ASYNC_TRACE_BUFFER_SIZE;

			// if smaller than next available index
			if (LastAvailableIndex <= TraceIndex)
			{
				// add one more buffer
				TraceData.Add(MakeUnique<TTraceThreadData<DatumType>>());
			}

			FTraceHandle Result(State.CurrentFrame, TraceIndex);
			FBufferIndexPair(TraceIndex).DatumLookupChecked(TraceData) = Val;

			ExecuteAsyncTraceIfAvailable<DatumType>(State, false);

			++TraceIndex;

			return Result;
		}
	}
}

FWorldAsyncTraceState::FWorldAsyncTraceState()
	: CurrentFrame             (0)
{
	// initial buffer is open for business
	DataBuffer[CurrentFrame].bAsyncAllowed = true;
}

FTraceHandle UWorld::AsyncLineTraceByChannel(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */, const FTraceDelegate* InDelegate/* = nullptr */, uint32 UserData /* = 0 */)
{
	return StartNewTrace(this, AsyncTraceState, FTraceDatum(this, FCollisionShape::LineShape, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam, TraceChannel, UserData, InTraceType, Start, End, FQuat::Identity, InDelegate, AsyncTraceState.CurrentFrame));
}

FTraceHandle UWorld::AsyncLineTraceByObjectType(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FTraceDelegate* InDelegate/* = nullptr */, uint32 UserData /* = 0 */)
{
	return StartNewTrace(this, AsyncTraceState, FTraceDatum(this, FCollisionShape::LineShape, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams, DefaultCollisionChannel, UserData, InTraceType, Start, End, FQuat::Identity, InDelegate, AsyncTraceState.CurrentFrame));
}

FTraceHandle UWorld::AsyncLineTraceByProfile(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, FName ProfileName, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FTraceDelegate* InDelegate/* = nullptr */, uint32 UserData /* = 0 */)
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return StartNewTrace(this, AsyncTraceState, FTraceDatum(this, FCollisionShape::LineShape, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam, TraceChannel, UserData, InTraceType, Start, End, FQuat::Identity, InDelegate, AsyncTraceState.CurrentFrame));
}

FTraceHandle UWorld::AsyncSweepByChannel(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */, const FTraceDelegate* InDelegate/* = nullptr */, uint32 UserData /* = 0 */)
{
	return StartNewTrace(this, AsyncTraceState, FTraceDatum(this, CollisionShape, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam, TraceChannel, UserData, InTraceType, Start, End, Rot, InDelegate, AsyncTraceState.CurrentFrame));
}

FTraceHandle UWorld::AsyncSweepByObjectType(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FTraceDelegate* InDelegate/* = nullptr */, uint32 UserData /* = 0 */)
{
	return StartNewTrace(this, AsyncTraceState, FTraceDatum(this, CollisionShape, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams, DefaultCollisionChannel, UserData, InTraceType, Start, End, Rot, InDelegate, AsyncTraceState.CurrentFrame));
}

FTraceHandle UWorld::AsyncSweepByProfile(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FTraceDelegate* InDelegate/* = nullptr */, uint32 UserData /* = 0 */)
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return StartNewTrace(this, AsyncTraceState, FTraceDatum(this, CollisionShape, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam, TraceChannel, UserData, InTraceType, Start, End, Rot, InDelegate, AsyncTraceState.CurrentFrame));
}

// overlap functions
FTraceHandle UWorld::AsyncOverlapByChannel(const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FCollisionResponseParams& ResponseParam /* = FCollisionResponseParams::DefaultResponseParam */, const FOverlapDelegate* InDelegate/* = nullptr */, uint32 UserData /* = 0 */)
{
	return StartNewTrace(this, AsyncTraceState, FOverlapDatum(this, CollisionShape, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam, TraceChannel, UserData, Pos, Rot, InDelegate, AsyncTraceState.CurrentFrame));
}

FTraceHandle UWorld::AsyncOverlapByObjectType(const FVector& Pos, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FOverlapDelegate* InDelegate/* = nullptr */, uint32 UserData /* = 0 */)
{
	return StartNewTrace(this, AsyncTraceState, FOverlapDatum(this, CollisionShape, Params, FCollisionResponseParams::DefaultResponseParam, ObjectQueryParams, DefaultCollisionChannel, UserData, Pos, Rot, InDelegate, AsyncTraceState.CurrentFrame));
}

FTraceHandle UWorld::AsyncOverlapByProfile(const FVector& Pos, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params /* = FCollisionQueryParams::DefaultQueryParam */, const FOverlapDelegate* InDelegate /* = nullptr */, uint32 UserData /* = 0 */)
{
	ECollisionChannel TraceChannel;
	FCollisionResponseParams ResponseParam;
	GetCollisionProfileChannelAndResponseParams(ProfileName, TraceChannel, ResponseParam);

	return StartNewTrace(this, AsyncTraceState, FOverlapDatum(this, CollisionShape, Params, ResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam, TraceChannel, UserData, Pos, Rot, InDelegate, AsyncTraceState.CurrentFrame));
}

bool UWorld::IsTraceHandleValid(const FTraceHandle& Handle, bool bOverlapTrace)
{
	// only valid if it's previous frame or current frame
	if (Handle._Data.FrameNumber != AsyncTraceState.CurrentFrame - 1 && Handle._Data.FrameNumber != AsyncTraceState.CurrentFrame)
	{
		return false;
	}

	// make sure it has valid index
	AsyncTraceData& DataBuffer = AsyncTraceState.GetBufferForFrame(Handle._Data.FrameNumber);

	// this function basically verifies if the address location 
	// is VALID, not necessarily that location was USED in that frame
	FBufferIndexPair Loc(Handle._Data.Index);
	if (bOverlapTrace)
	{
		return !!Loc.DatumLookup(Handle.IsTransactional() ? DataBuffer.TransactionalOverlapData.Data : DataBuffer.OverlapData);
	}
	else
	{
		return !!Loc.DatumLookup(Handle.IsTransactional() ? DataBuffer.TransactionalTraceData.Data : DataBuffer.TraceData);
	}
}

bool UWorld::QueryTraceData(const FTraceHandle& Handle, FTraceDatum& OutData)
{
	// valid if previous frame request
	if (Handle._Data.FrameNumber != AsyncTraceState.CurrentFrame - 1)
	{
		return false;
	}

	AsyncTraceData& DataBuffer = AsyncTraceState.GetBufferForPreviousFrame();
	if (!DataBuffer.bAsyncTasksCompleted)
	{
		return false;
	}

	TArray<TUniquePtr<TTraceThreadData<FTraceDatum>>>& TraceData = Handle.IsTransactional() ? DataBuffer.TransactionalTraceData.Data : DataBuffer.TraceData;
	if (auto* Data = FBufferIndexPair(Handle._Data.Index).DatumLookup(TraceData))
	{
		OutData = *Data;
		return true;
	}

	return false;
}

bool UWorld::QueryOverlapData(const FTraceHandle& Handle, FOverlapDatum& OutData)
{
	if (Handle._Data.FrameNumber != AsyncTraceState.CurrentFrame - 1)
	{
		return false;
	}

	AsyncTraceData& DataBuffer = AsyncTraceState.GetBufferForPreviousFrame();
	if (!DataBuffer.bAsyncTasksCompleted)
	{
		return false;
	}
	TArray<TUniquePtr<TTraceThreadData<FOverlapDatum>>>& OverlapData = Handle.IsTransactional() ? DataBuffer.TransactionalOverlapData.Data : DataBuffer.OverlapData;
	if (auto* Data = FBufferIndexPair(Handle._Data.Index).DatumLookup(OverlapData))
	{
		OutData = *Data;
		return true;
	}

	return false;
}

AsyncTraceData* GetTraceDataForFrame(FWorldAsyncTraceState& AsyncTraceState, int32 FrameNumber)
{
	if (FrameNumber == AsyncTraceState.CurrentFrame)
	{
		return &AsyncTraceState.GetBufferForCurrentFrame();
	}
	else if (FrameNumber == AsyncTraceState.CurrentFrame - 1)
	{
		return &AsyncTraceState.GetBufferForPreviousFrame();
	}
	return nullptr;
}

void UWorld::AddTraceData(const FTraceHandle& Handle, const TArray<FHitResult>& Results)
{
	AsyncTraceData* TraceData = GetTraceDataForFrame(AsyncTraceState, Handle._Data.FrameNumber);
	if (TraceData == nullptr)
	{
		return;
	}

	AsyncTraceData& DataBuffer = *TraceData;
	FBufferIndexPair Pair(Handle._Data.Index);
	if (FTraceDatum* Datum = Pair.DatumLookup(DataBuffer.TransactionalTraceData.Data))
	{
		Datum->OutHits.Append(Results);
	}
}

void UWorld::AddOverlapData(const FTraceHandle& Handle, const TArray<FOverlapResult>& Results)
{
	AsyncTraceData* TraceData = GetTraceDataForFrame(AsyncTraceState, Handle._Data.FrameNumber);
	if (TraceData == nullptr)
	{
		return;
	}

	AsyncTraceData& DataBuffer = *TraceData;
	FBufferIndexPair Pair(Handle._Data.Index);
	if (FOverlapDatum* Datum = Pair.DatumLookup(DataBuffer.TransactionalOverlapData.Data))
	{
		Datum->OutOverlaps.Append(Results);
	}
}

void UWorld::WaitForAllAsyncTraceTasks()
{
	const bool bRunAsyncTraceOnWorkerThread = AsyncTraceCVars::IsAsyncTraceOnWorkerThreads();
	if (bRunAsyncTraceOnWorkerThread)
	{
		// if running thread, wait until all threads finishes, if we don't do this, there might be more thread running
		AsyncTraceData& DataBufferExecuted = AsyncTraceState.GetBufferForPreviousFrame();
		if (DataBufferExecuted.AsyncTraceCompletionEvent.Num() > 0)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_WaitForAllAsyncTraceTasks);
			CSV_SCOPED_TIMING_STAT(WorldCollision, StatWaitForAllAsyncTraceTasks);
			FTaskGraphInterface::Get().WaitUntilTasksComplete(DataBufferExecuted.AsyncTraceCompletionEvent, ENamedThreads::GameThread);
			DataBufferExecuted.AsyncTraceCompletionEvent.Reset();
		}
	}
}

void UWorld::ResetAsyncTrace()
{
	AsyncTraceData& DataBufferExecuted = AsyncTraceState.GetBufferForPreviousFrame();

	// Wait for thread
	WaitForAllAsyncTraceTasks();
	DataBufferExecuted.bAsyncTasksCompleted = true;

#if !UE_BUILD_SHIPPING
	FAutoScopedDurationTimer Timer;
#endif //!UE_BUILD_SHIPPING

	// do run delegates before starting next round
	for (int32 Idx = 0; Idx != DataBufferExecuted.NumQueuedTraceData; ++Idx)
	{
		FTraceDatum& TraceData = FBufferIndexPair(Idx).DatumLookupChecked(DataBufferExecuted.TraceData);
		TraceData.Delegate.ExecuteIfBound(FTraceHandle(TraceData.FrameNumber, Idx), TraceData);
	}

	for (int32 Idx = 0; Idx != DataBufferExecuted.NumQueuedOverlapData; ++Idx)
	{
		FOverlapDatum& TraceData = FBufferIndexPair(Idx).DatumLookupChecked(DataBufferExecuted.OverlapData);
		TraceData.Delegate.ExecuteIfBound(FTraceHandle(TraceData.FrameNumber, Idx), TraceData);
	}

	for (int32 Idx = 0; Idx != DataBufferExecuted.TransactionalTraceData.NumQueued; ++Idx)
	{
		FTraceDatum& TraceData = FBufferIndexPair(Idx).DatumLookupChecked(DataBufferExecuted.TransactionalTraceData.Data);
		TraceData.Delegate.ExecuteIfBound(FTraceHandle(TraceData.FrameNumber, Idx, true), TraceData);
	}
	for (int32 Idx = 0; Idx != DataBufferExecuted.TransactionalOverlapData.NumQueued; ++Idx)
	{
		FOverlapDatum& OverlapData = FBufferIndexPair(Idx).DatumLookupChecked(DataBufferExecuted.TransactionalOverlapData.Data);
		OverlapData.Delegate.ExecuteIfBound(FTraceHandle(OverlapData.FrameNumber, Idx, true), OverlapData);
	}

#if !UE_BUILD_SHIPPING
	Timer.Stop();

	//If the delegate dispatch has taken longer than the defined threshold, log data about which delegates were fired
	const float AsyncTraceDelegateThresholdTimeSeconds = AsyncTraceCVars::AsyncTraceDelegateHitchThresholdMS / 1000.0f;
	if(AsyncTraceDelegateThresholdTimeSeconds > 0.0f && Timer.GetTime() > AsyncTraceDelegateThresholdTimeSeconds && FPlatformTime::Seconds() > AsyncDelegateHitchLoggingLastTimestamp + AsyncTraceCVars::AsyncTraceDelegateLoggingIntervalSeconds)
	{
		AsyncDelegateHitchLoggingLastTimestamp = FPlatformTime::Seconds();
		UE_LOGF(LogWorldCollision , Log, "ResetAsyncTrace has exceeded budget [Time:%f (s) Budget: %f (s) Over: %f] - Dumping Trace Delegates", Timer.GetTime(), AsyncTraceDelegateThresholdTimeSeconds, Timer.GetTime() - AsyncTraceDelegateThresholdTimeSeconds);

#if CSV_PROFILER
		const FCsvProfiler* CSVProfiler = FCsvProfiler::Get();

		if (CSVProfiler && CSVProfiler->IsCapturing())
		{
			UE_LOGF(LogWorldCollision, Log, "ResetAsyncTrace for CSVFrame %d", CSVProfiler->GetCaptureFrameNumber());
		}
#endif //CSV_PROFILER

		for (int32 Idx = 0; Idx != DataBufferExecuted.NumQueuedTraceData; ++Idx)
		{
			const FTraceDatum& TraceData = FBufferIndexPair(Idx).DatumLookupChecked(DataBufferExecuted.TraceData);
			const FName TraceTag = TraceData.CollisionParams.CollisionQueryParam.TraceTag;
			const UObject* DelegateObject = TraceData.Delegate.GetUObject();
			UE_LOGF(LogWorldCollision, Log, "--  Trace  -- TraceTag: %ls Object: %ls", *TraceTag.ToString(), DelegateObject ? *DelegateObject->GetName() : TEXT("nullptr"));
		}

		for (int32 Idx = 0; Idx != DataBufferExecuted.NumQueuedOverlapData; ++Idx)
		{
			const FOverlapDatum& TraceData = FBufferIndexPair(Idx).DatumLookupChecked(DataBufferExecuted.OverlapData);
			const FName TraceTag = TraceData.CollisionParams.CollisionQueryParam.TraceTag;
			const UObject* DelegateObject = TraceData.Delegate.GetUObject();
			UE_LOGF(LogWorldCollision, Log, "-- Overlap -- TraceTag: %ls Object: %ls", *TraceTag.ToString(), DelegateObject ? *DelegateObject->GetName() : TEXT("nullptr"));
		}
	}
#endif //!UE_BUILD_SHIPPING
}

void UWorld::FinishAsyncTrace()
{
	// execute all remainder
	ExecuteAsyncTraceIfAvailable<FTraceDatum>  (AsyncTraceState, true);
	ExecuteAsyncTraceIfAvailable<FOverlapDatum>(AsyncTraceState, true);

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	auto DoWorkCallback = [this]()
	{
		// We need to add all requests inside of a transaction, otherwise when a transaction is 
		// aborted because other work needs to be processed we would lose all active queries.
		if (!PhysicsQueryHandler->AreAsyncRequestsAdded())
		{
			ExecuteAsyncTransactionalTraceIfAvailable<FTraceDatum>(this, AsyncTraceState, true);
			ExecuteAsyncTransactionalTraceIfAvailable<FOverlapDatum>(this, AsyncTraceState, true);
		}

		// Make sure all requests are done, otherwise abort. If this transaction succeeds then it will commit back the results.
		PhysicsQueryHandler->VerifyAsyncRequestsAreCompletedOrAbort();
	};
	if (PhysicsQueryHandler != nullptr)
	{
		static FName TransactionalWorkName("UWorld::FinishAsyncTrace");
		UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, DoWorkCallback);
	}
#endif

	// this flag only needed to know I can't accept any more new request in current frame
	AsyncTraceState.GetBufferForCurrentFrame().bAsyncAllowed = false;

	// increase buffer index to next one
	++AsyncTraceState.CurrentFrame;

	// set up new buffer to accept trace requests
	AsyncTraceData& NewAsyncBuffer = AsyncTraceState.GetBufferForCurrentFrame();
	NewAsyncBuffer.bAsyncAllowed = true;
	NewAsyncBuffer.NumQueuedTraceData = 0;
	NewAsyncBuffer.NumQueuedOverlapData = 0;

	NewAsyncBuffer.TransactionalTraceData.NumQueued = 0;
	NewAsyncBuffer.TransactionalOverlapData.NumQueued = 0;

	NewAsyncBuffer.bAsyncTasksCompleted = false;

}

