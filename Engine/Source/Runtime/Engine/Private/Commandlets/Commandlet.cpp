// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/Commandlet.h"
#include "Engine/World.h"
#include "Misc/AutomationTest.h"

#include "ComputeWorkerInterface.h"
#include "Engine/Engine.h"
#include "EngineModule.h"
#include "Misc/App.h"
#include "RenderingThread.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphEvent.h"
#include "SceneInterface.h"
#include "Misc/CoreDelegates.h"
#include "RendererInterface.h"
#include "HAL/ThreadManager.h"
#include "Framework/Application/SlateApplication.h"

#if WITH_EDITOR
#include "Kismet2/BlueprintEditorUtils.h"
#endif //  WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(Commandlet)

DEFINE_LOG_CATEGORY_STATIC(LogCommandlet, Log, All);

UCommandlet::UCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	IsServer = true;
	IsClient = true;
	IsEditor = true;
	ShowErrorCount = true;
	ShowProgress = true;
	FastExit = false;
	UseCommandletResultAsExitCode = false;
}

#if WITH_EDITOR
bool UCommandlet::ApplyCommandLineSwitches( UObject* Object, const TArray<FString>& Switches )
{
	for (const FString& Switch : Switches)
	{
		FString Key;
		FString Value;

		if (!Switch.Split(TEXT("="), &Key, &Value))
		{
			Key = Switch;
		}

		// Lookup property
		const FProperty* Property = Object->GetClass()->FindPropertyByName(*Key);

		// If we can't find the property, try for properties with the 'b' prefix
		if (!Property)
		{
			Key = TEXT("b") + Key;
			Property = Object->GetClass()->FindPropertyByName(*Key);
		}

		if (Property)
		{
			// If the property is a bool, treat no values as true
			if (Property->IsA(FBoolProperty::StaticClass()) && Value.IsEmpty())
			{
				Value = TEXT("True");
			}

			// Apply the property
			if (!FBlueprintEditorUtils::PropertyValueFromString(Property, Value, (uint8*)Object, nullptr))
			{
				UE_LOGF(LogCommandlet, Error, "Cannot set value for '%ls': '%ls'", *Key, *Value);
				return false;
			}
		}
	}

	return true;
}
#endif //  WITH_EDITOR

/* 
	Tests for Commandlet Utils
*/

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCommandletCommandLineParsingTest, "System.Commandlet.ParseCommandLine", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FCommandletCommandLineParsingTest::RunTest(const FString& Parameters)
{
	const TCHAR* TestCommandLine = TEXT("token1 token2 -switch1 -switch2 -NakedValue=Value -QuotedValue=\"Value\" -EmptyValue= -ValueWithAssignment=Foo=Bar");

	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Params;
	UCommandlet::ParseCommandLine(TestCommandLine, Tokens, Switches, Params);

	TestEqual(TEXT("ExpectedTokenCount"), Tokens.Num(), 2);
	TestEqual(TEXT("ExpectedSwitchCount"), Switches.Num(), 2);
	TestEqual(TEXT("ExpectedParamCount"), Params.Num(), 4);

	TestTrue(TEXT("tonen1 found"), Tokens.Contains(TEXT("token1")));
	TestTrue(TEXT("token2 found"), Tokens.Contains(TEXT("token2")));

	TestTrue(TEXT("switch1 set"), Switches.Contains(TEXT("switch1")));
	TestTrue(TEXT("switch1 set"), Switches.Contains(TEXT("switch2")));

	TestTrue(TEXT("NakedValue parsed"), Params.Contains(TEXT("NakedValue")));
	TestTrue(TEXT("QuotedValue parsed"), Params.Contains(TEXT("QuotedValue")));
	TestTrue(TEXT("EmptyValue parsed"), Params.Contains(TEXT("EmptyValue")));
	TestTrue(TEXT("ValueWithAssignment parsed"), Params.Contains(TEXT("ValueWithAssignment")));

	TestEqual(TEXT("NakedValue Correct"), Params.FindRef(TEXT("NakedValue")), TEXT("Value"));
	TestEqual(TEXT("QuotedValue Correct"), Params.FindRef(TEXT("QuotedValue")), TEXT("Value"));
	TestEqual(TEXT("EmptyValue Correct"), Params.FindRef(TEXT("EmptyValue")), TEXT(""));
	TestEqual(TEXT("ValueWithAssignment Correct"), Params.FindRef(TEXT("ValueWithAssignment")), TEXT("Foo=Bar"));

	return !HasAnyErrors();
} 

#endif // WITH_DEV_AUTOMATION_TESTS


void CommandletHelpers::TickEngine(UWorld* InWorld, double InDeltaTime)
{
	// Simulate an engine frame tick
	// Will make sure systems can perform their internal bookkeeping properly. For example, the VT system needs to 
	// process deleted VTs.

	FApp::SetDeltaTime(InDeltaTime);

	// Tick the engine.
	GEngine->Tick(FApp::GetDeltaTime(), false);

	// Tick Slate.
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().PumpMessages();
		FSlateApplication::Get().Tick();
	}

	// Required for FTimerManager to function - as it blocks ticks, if the frame counter doesn't change
	GFrameCounter++;

	// Update task graph.
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

	// Ticks all fake threads and their runnable objects.
	FThreadManager::Get().Tick();

	// Core ticker tick.
	FTSTicker::GetCoreTicker().Tick(FApp::GetDeltaTime());

	// Execute deferred commands.
	GEngine->TickDeferredCommands();

	// Tick rendering.
	if (InWorld && IsAllowCommandletRendering())
	{
		if (FSceneInterface* Scene = InWorld->Scene)
		{
			// BeingFrame/EndFrame (taken from FEngineLoop)

			ENQUEUE_RENDER_COMMAND(BeginFrame)([](FRHICommandListImmediate& RHICmdList)
			{
				GFrameNumberRenderThread++;
				FCoreDelegates::OnBeginFrameRT.Broadcast();
			});

			// Submit Compute Framework work (ComputeTaskExecutionGroup::EndOfFrameUpdate)
			const ERHIFeatureLevel::Type FeatureLevel = Scene->GetFeatureLevel();
			TArray<IComputeTaskWorker*> ComputeWorkers;
			Scene->GetComputeTaskWorkers(ComputeWorkers);

			ENQUEUE_RENDER_COMMAND(ComputeTaskWorker_EndOfFrameUpdate)([FeatureLevel, ComputeWorkers, Scene](FRHICommandListImmediate& RHICmdList)
			{				
				FRDGBuilder GraphBuilder(RHICmdList, FRDGEventName(TEXT("ComputeTaskWorker_EndOfFrameUpdate")));
				
				for (IComputeTaskWorker* ComputeTaskWorker : ComputeWorkers)
				{
					if (ComputeTaskWorker->HasWork(ComputeTaskExecutionGroup::EndOfFrameUpdate))
					{
						FComputeContext Context
						{
							.GraphBuilder = GraphBuilder,
							.ExecutionGroupName = ComputeTaskExecutionGroup::EndOfFrameUpdate,
							.FeatureLevel = FeatureLevel,
							.Scene = Scene,
						};

						ComputeTaskWorker->SubmitWork(Context);
					}
				}

				GraphBuilder.Execute();
			});

			ENQUEUE_RENDER_COMMAND(EndFrame)([](FRHICommandListImmediate& RHICmdList)
			{
				FCoreDelegates::OnEndFrameRT.Broadcast();
				RHICmdList.EndFrame();
			});

			FlushRenderingCommands();
		}

		ENQUEUE_RENDER_COMMAND(VirtualTextureScalability_Release)([](FRHICommandList& RHICmdList)
		{
			GetRendererModule().ReleaseVirtualTexturePendingResources();
		});
	}
}
