// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/CommandletHelpers.h"

#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"

namespace CommandletHelpers
{
	FString BuildCommandletProcessArguments(const TCHAR* const CommandletName, const TCHAR* const ProjectPath, const TCHAR* const AdditionalArguments)
	{
		check(CommandletName && FCString::Strlen(CommandletName) > 0);

		FString Arguments;

		if (ProjectPath && FCString::Strlen(ProjectPath) > 0)
		{
			if (!Arguments.IsEmpty())
			{
				Arguments += TEXT(" ");
			}
			Arguments += ProjectPath;
		}

		if (!Arguments.IsEmpty())
		{
			Arguments += TEXT(" ");
		}
		Arguments += TEXT("-run=");
		Arguments += CommandletName;

		if (AdditionalArguments && FCString::Strlen(AdditionalArguments) > 0)
		{
			if (!Arguments.IsEmpty())
			{
				Arguments += TEXT(" ");
			}
			Arguments += AdditionalArguments;
		}

		return Arguments;
	}
}
