// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/PluginCommandlet.h"
#include "Misc/Paths.h"

#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PluginCommandlet)


DEFINE_LOG_CATEGORY_STATIC(LogPluginCommandlet, Log, All);

/**
 * UPluginCommandlet
 *
 * Commandlet used for enabling/disabling plugins
 *
 * Usage:
 *	Plugin Enable/Disable PluginName,PluginName2
 *
 * Optional parameters:
 *	-Force: Forces a plugin to be enabled/disabled, even if it can not be found.
 *
 * Example:
 *	Plugin Enable NetcodeUnitTest
 *	Plugin Enable NetcodeUnitTest,NUTUnrealEngine
 */

UPluginCommandlet::UPluginCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	IsClient = false;
	IsServer = false;
	IsEditor = false;
}

int32 UPluginCommandlet::Main(const FString& Params)
{
	bool bSuccess = false;

#if !UE_BUILD_SHIPPING
	const TCHAR* ParamStr = *Params;
	ParseCommandLine(ParamStr, CmdLineTokens, CmdLineSwitches);

	if (CmdLineTokens.Num() > 0)
	{
		bool bEnable = false;

		int32 CommandIdx = CmdLineTokens.IndexOfByPredicate(
			[&bEnable](const FString& CurToken)
			{
				bool bReturnVal = false;

				if (CurToken == TEXT("Enable") || CurToken == TEXT("Disable"))
				{
					bEnable = CurToken == TEXT("Enable");
					bReturnVal = true;
				}

				return bReturnVal;
			});

		if (CommandIdx != INDEX_NONE)
		{
			// Trim all tokens up to the command, to keep things simple
			CmdLineTokens.RemoveAt(0, CommandIdx+1);

			FString PluginList = (CmdLineTokens.Num() > 0 ? CmdLineTokens.Pop() : TEXT(""));
			TArray<FString> PluginArray;

			PluginList.ParseIntoArray(PluginArray, TEXT(","));

			if (PluginArray.Num() > 0 && IProjectManager::Get().LoadProjectFile(FPaths::GetProjectFilePath()))
			{
				FText FailReason;

				bSuccess = true;

				for (FString PluginName : PluginArray)
				{
					bool bSetPluginState = false;

					if (IPluginManager::Get().FindPlugin(PluginName).IsValid() || CmdLineSwitches.Contains(TEXT("FORCE")))
					{
						bSetPluginState = IProjectManager::Get().SetPluginEnabled(PluginName, bEnable, FailReason);

						// Try to save the project file if needed
						if (IProjectManager::Get().IsCurrentProjectDirty())
						{
							if (!IProjectManager::Get().SaveCurrentProjectToDisk(FailReason))
							{
								bSetPluginState = false;
							}
						}
					}
					else
					{
						FailReason = FText::FromString(TEXT("Plugin not found. Add -Force to the commandline, to override."));
					}

					if (bSetPluginState)
					{
						UE_LOGF(LogPluginCommandlet, Log, "Successfully %ls plugin '%ls'",
								(bEnable ? TEXT("enabled") : TEXT("disabled")), *PluginName);
					}
					else
					{
						UE_LOGF(LogPluginCommandlet, Error, "Failed to %ls plugin '%ls' - error: %ls",
								(bEnable ? TEXT("enable") : TEXT("disable")), *PluginName, *FailReason.ToString());

						bSuccess = false;
					}
				}
			}
			else if (PluginArray.Num() == 0)
			{
				UE_LOGF(LogPluginCommandlet, Error, "Failed to parse a plugin name.");
				bSuccess = false;
			}
			else
			{
				UE_LOGF(LogPluginCommandlet, Error, "Failed to load project file.");
				bSuccess = false;
			}
		}
		else
		{
			UE_LOGF(LogPluginCommandlet, Error, "No command specified or unknown command.");
			bSuccess = false;
		}
	}
	else
	{
		UE_LOGF(LogPluginCommandlet, Error, "No command specified. Example: 'PluginCommandlet Enable NetcodeUnitTest'");
		bSuccess = false;
	}
#else
	UE_LOGF(LogPluginCommandlet, Error, "Plugin commandlet disabled in shipping mode.");
	bSuccess = false;
#endif

	return bSuccess ? 0 : 1;
}

