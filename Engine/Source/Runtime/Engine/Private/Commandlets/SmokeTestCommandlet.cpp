// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SmokeTestCommandlet.cpp: Commandled used for smoke testing.

=============================================================================*/

#include "Commandlets/SmokeTestCommandlet.h"
#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SmokeTestCommandlet)

USmokeTestCommandlet::USmokeTestCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	IsClient = false;
	IsEditor = false;
	LogToConsole = true;
}

int32 USmokeTestCommandlet::Main( const FString& Params )
{
	const TCHAR* Parms = *Params;

	RequestEngineExit(TEXT("SmokeTestCommandlet Main")); // so CTRL-C will exit immediately
	bool bAllSuccessful = FAutomationTestFramework::Get().RunSmokeTests();

	return bAllSuccessful ? 0 : 1;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRunServer, "System.Core.Misc.Run Server", EAutomationTestFlags::CommandletContext | EAutomationTestFlags::SmokeFilter)

bool FRunServer::RunTest(const FString& Parameters)
{
	// put various smoke test testing code in here before moving off to their
	// own commandlet
	if( GEngine && IsRunningDedicatedServer() )
	{
		// Update FApp::CurrentTime / FApp::DeltaTime while taking into account max tick rate.
		GEngine->UpdateTimeAndHandleMaxTickRate();

		// Tick the engine.
		GEngine->Tick( FApp::GetDeltaTime(), false );
	}

	return true;
}



