// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ComponentDelegateBinding.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentDelegateBinding)

UComponentDelegateBinding::UComponentDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UComponentDelegateBinding::BindDynamicDelegates(UObject* InInstance) const
{
	for(int32 BindIdx=0; BindIdx<ComponentDelegateBindings.Num(); BindIdx++)
	{
		const FBlueprintComponentDelegateBinding& Binding = ComponentDelegateBindings[BindIdx];

	// Get the property that points to the component
		if (const FObjectProperty* ObjProp = FindFProperty<FObjectProperty>(InInstance->GetClass(), Binding.ComponentPropertyName))
		{
			// ..see if there is actually a component assigned
			if (UObject* Component = ObjProp->GetObjectPropertyValue_InContainer(InInstance))
			{
			// If there is, find and return the delegate property on it
				if (FMulticastDelegateProperty* MulticastDelegateProp = FindFProperty<FMulticastDelegateProperty>(Component->GetClass(), Binding.DelegatePropertyName))
				{
					// Get the function we want to bind
					if (UFunction* FunctionToBind = InInstance->GetClass()->FindFunctionByName(Binding.FunctionNameToBind))
					{
						// Bind function on the instance to this delegate
						FScriptDelegate Delegate;
						Delegate.BindUFunction(InInstance, Binding.FunctionNameToBind);
						MulticastDelegateProp->AddDelegate(MoveTemp(Delegate), Component);
					}
				}
			}
		}
	}
}

void UComponentDelegateBinding::UnbindDynamicDelegates(UObject* InInstance) const
{
	for (int32 BindIdx = 0; BindIdx<ComponentDelegateBindings.Num(); BindIdx++)
	{
		const FBlueprintComponentDelegateBinding& Binding = ComponentDelegateBindings[BindIdx];

		// Get the property that points to the component
		if (const FObjectProperty* ObjProp = FindFProperty<FObjectProperty>(InInstance->GetClass(), Binding.ComponentPropertyName))
		{
			// ..see if there is actually a component assigned
			if (UObject* Component = ObjProp->GetObjectPropertyValue_InContainer(InInstance))
			{
				// If there is, find and return the delegate property on it
				if (FMulticastDelegateProperty* MulticastDelegateProp = FindFProperty<FMulticastDelegateProperty>(Component->GetClass(), Binding.DelegatePropertyName))
		{
					// Unbind function on the instance to this delegate
					FScriptDelegate Delegate;
					Delegate.BindUFunction(InInstance, Binding.FunctionNameToBind);
					MulticastDelegateProp->RemoveDelegate(Delegate, Component);
				}
			}
		}
	}
}

void UComponentDelegateBinding::UnbindDynamicDelegatesForProperty(UObject* InInstance, const FObjectProperty* InObjectProperty) const
{
	for(int32 BindIdx=0; BindIdx<ComponentDelegateBindings.Num(); BindIdx++)
	{
		const FBlueprintComponentDelegateBinding& Binding = ComponentDelegateBindings[BindIdx];
		if (InObjectProperty->GetFName() == Binding.ComponentPropertyName)
		{
			const FObjectProperty* ObjProp = FindFProperty<FObjectProperty>(InInstance->GetClass(), Binding.ComponentPropertyName);
			if (ObjProp == InObjectProperty)
			{
				// ..see if there is actually a component assigned
				if (UObject* Component = ObjProp->GetObjectPropertyValue_InContainer(InInstance))
				{
					// If there is, find and return the delegate property on it
					if (FMulticastDelegateProperty* MulticastDelegateProp = FindFProperty<FMulticastDelegateProperty>(Component->GetClass(), Binding.DelegatePropertyName))
					{
						// Unbind function on the instance from this delegate
						FScriptDelegate Delegate;
						Delegate.BindUFunction(InInstance, Binding.FunctionNameToBind);
						MulticastDelegateProp->RemoveDelegate(Delegate, Component);
					}
				}
				break;
			}
		}
	}
}

