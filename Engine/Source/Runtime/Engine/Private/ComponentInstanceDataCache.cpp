// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentInstanceDataCache.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/DuplicatedObject.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "UObject/Package.h"
#include "UObject/UObjectAnnotation.h"
#include "GameFramework/Actor.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentInstanceDataCache)

class FDataCachePropertyWriter : public FObjectWriter
{
public:
	FDataCachePropertyWriter(const UObject* InCacheObject, FInstanceCacheDataBase& InInstanceData)
		: FObjectWriter(InInstanceData.SavedProperties)
		, CacheObject(InCacheObject)
		, InstanceData(InInstanceData)
	{
		// Include properties that would normally skip tagged serialization (e.g. bulk serialization of array properties).
		ArPortFlags |= PPF_ForceTaggedSerialization;

		// Nested subobjects should be recursed in to
		ArPortFlags |= PPF_DeepCompareInstances;
	}

	void SerializeProperties()
	{
		if (CacheObject)
		{
			UClass* CacheObjectClass = CacheObject->GetClass();
			CacheObjectClass->SerializeTaggedProperties(*this, (uint8*)CacheObject, CacheObjectClass, (uint8*)CacheObject->GetArchetype());

			// Sort duplicated objects, if any, this ensures that lower depth duplicated objects are first in the array, which ensures proper creation order when deserializing
			if (InstanceData.DuplicatedObjects.Num() > 0)
			{
				InstanceData.DuplicatedObjects.StableSort([](const FDataCacheDuplicatedObjectData& One, const FDataCacheDuplicatedObjectData& Two) -> bool
				{
					return One.ObjectPathDepth < Two.ObjectPathDepth;
				});
			}
		}
	}

	virtual ~FDataCachePropertyWriter()
	{
		DuplicatedObjectAnnotation.RemoveAllAnnotations();
	}

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
	{
		// Immutable structs expect to serialize all properties so don't skip regardless of other conditions
		UScriptStruct* ScriptStruct = InProperty->GetOwner<UScriptStruct>();
		const bool bPropertyInImmutableStruct = ScriptStruct && ((ScriptStruct->StructFlags & STRUCT_Immutable) != 0);

		return (!bPropertyInImmutableStruct
			&& (InProperty->HasAnyPropertyFlags(CPF_Transient)
				|| !InProperty->HasAnyPropertyFlags(CPF_Edit | CPF_Interp)
				|| InProperty->IsA<FMulticastDelegateProperty>()
				|| PropertiesToSkip.Contains(InProperty)
				)
			);
	}

	UObject* GetDuplicatedObject(UObject* Object)
	{
		UObject* Result = Object;
		if (IsValid(Object))
		{
			// Check for an existing duplicate of the object.
			FDuplicatedObject DupObjectInfo = DuplicatedObjectAnnotation.GetAnnotation( Object );
			if( !DupObjectInfo.IsDefault() )
			{
				Result = DupObjectInfo.DuplicatedObject.GetEvenIfUnreachable();
			}
			else if (Object->GetOuter() == CacheObject)
			{
				Result = DuplicateObject(Object, InstanceData.GetUniqueTransientObject(Object->GetClass()));
				InstanceData.DuplicatedObjects.Emplace(Result);
			}
			else
			{
				check(Object->IsIn(CacheObject));

				// Check to see if the object's outer is being duplicated.
				UObject* DupOuter = GetDuplicatedObject(Object->GetOuter());
				if (DupOuter != nullptr)
				{
					// First check if the duplicated outer already has an allocated duplicate of this object
					Result = static_cast<UObject*>(FindObjectWithOuter(DupOuter, Object->GetClass(), Object->GetFName()));

					if (Result == nullptr)
					{
						// The object's outer is being duplicated, create a duplicate of this object.
						Result = DuplicateObject(Object, DupOuter);
					}

					DuplicatedObjectAnnotation.AddAnnotation( Object, FDuplicatedObject( Result ) );
				}
			}
		}

		return Result;
	}

	virtual FArchive& operator<<(FName& Name) override
	{
		// store the reference to this name in the array instead of the global table, this allow to for persistence
		int32 ReferenceIndex = InstanceData.ReferencedNames.AddUnique(Name);
		// save the name as an index in the referenced name array
		*this << ReferenceIndex;

		return *this;
	}

	virtual FArchive& operator<<(UObject*& Object) override
	{
		UObject* SerializedObject = Object;
		if (Object && CacheObject && Object->IsIn(CacheObject))
		{
			SerializedObject = GetDuplicatedObject(Object);
		}

		// store the pointer to this object
		int32 ReferenceIndex = INDEX_NONE;
		if (SerializedObject)
		{
			ReferenceIndex = InstanceData.ReferencedObjects.Num();
			InstanceData.ReferencedObjects.Add(SerializedObject);
		}
		// save the pointer as an index in the referenced object array
		*this << ReferenceIndex;

		return *this;
	}

	virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override
	{
		UObject* Obj = LazyObjectPtr.Get();
		*this << Obj;
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& ObjectPtr) override
	{
		return FArchiveUObject::SerializeObjectPtr(*this, ObjectPtr);
	}

protected:
	TSet<const FProperty*> PropertiesToSkip;

private:
	const UObject* CacheObject;
	FDuplicatedObjectAnnotation DuplicatedObjectAnnotation;
	FInstanceCacheDataBase& InstanceData;
};

class FComponentPropertyWriter : public FDataCachePropertyWriter
{
public:

	FComponentPropertyWriter(const UActorComponent* Component, FActorComponentInstanceData& InInstanceData)
		: FDataCachePropertyWriter(Component, InInstanceData)
	{
		if (Component)
		{
			Component->GetUCSModifiedProperties(PropertiesToSkip);

			if (AActor* ComponentOwner = Component->GetOwner())
			{
				// If this is the owning Actor's root scene component, don't include relative transform properties. This is handled elsewhere.
				if (Component == ComponentOwner->GetRootComponent())
				{
					UClass* ComponentClass = Component->GetClass();
					PropertiesToSkip.Add(ComponentClass->FindPropertyByName(USceneComponent::GetRelativeLocationPropertyName()));
					PropertiesToSkip.Add(ComponentClass->FindPropertyByName(USceneComponent::GetRelativeRotationPropertyName()));
					PropertiesToSkip.Add(ComponentClass->FindPropertyByName(USceneComponent::GetRelativeScale3DPropertyName()));
				}
			}

			SerializeProperties();
		}
	}
};

class FActorPropertyWriter : public FDataCachePropertyWriter
{
public:

	FActorPropertyWriter(const AActor* Actor, FActorInstanceData& InInstanceData)
		: FDataCachePropertyWriter(Actor, InInstanceData)
	{
		if (Actor)
		{
			SerializeProperties();
		}
	}
};

class FDataCachePropertyReader : public FObjectReader
{
public:
	FDataCachePropertyReader(FInstanceCacheDataBase& InInstanceData)
		: FObjectReader(InInstanceData.SavedProperties)
		, InstanceData(InInstanceData)
	{
		// Include properties that would normally skip tagged serialization (e.g. bulk serialization of array properties).
		ArPortFlags |= PPF_ForceTaggedSerialization;
	}

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
	{
		return PropertiesToSkip.Contains(InProperty);
	}

	virtual FArchive& operator<<(FName& Name) override
	{
		// FName are serialized as Index in ActorInstanceData instead of the normal FName table
		int32 ReferenceIndex = INDEX_NONE;
		*this << ReferenceIndex;
		Name = InstanceData.ReferencedNames.IsValidIndex(ReferenceIndex) ? InstanceData.ReferencedNames[ReferenceIndex] : FName();
		return *this;
	}

	virtual FArchive& operator<<(UObject*& Object) override
	{
		// UObject pointer are serialized as Index in ActorInstanceData
		int32 ReferenceIndex = INDEX_NONE;
		*this << ReferenceIndex;
		Object = InstanceData.ReferencedObjects.IsValidIndex(ReferenceIndex) ? ToRawPtr(InstanceData.ReferencedObjects[ReferenceIndex]) : nullptr;
		return *this;
	}

	virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override
	{
		UObject* Obj = LazyObjectPtr.Get();
		*this << Obj;
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& ObjectPtr) override
	{
		return FArchiveUObject::SerializeObjectPtr(*this, ObjectPtr);
	}

	FInstanceCacheDataBase& InstanceData;
	TSet<const FProperty*> PropertiesToSkip;
};

class FComponentPropertyReader : public FDataCachePropertyReader
{
public:
	FComponentPropertyReader(UActorComponent* InComponent, FActorComponentInstanceData& InInstanceData)
		: FDataCachePropertyReader(InInstanceData)
	{
		InComponent->GetUCSModifiedProperties(PropertiesToSkip);

		UClass* Class = InComponent->GetClass();
		Class->SerializeTaggedProperties(*this, (uint8*)InComponent, Class, (uint8*)InComponent->GetArchetype());
	}
};

class FActorPropertyReader : public FDataCachePropertyReader
{
public:
	FActorPropertyReader(AActor* InActor, FActorInstanceData& InInstanceData)
		: FDataCachePropertyReader(InInstanceData)
	{
		UClass* Class = InActor->GetClass();
		Class->SerializeTaggedProperties(*this, (uint8*)InActor, Class, (uint8*)InActor->GetArchetype());
	}

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
	{
		// We want to skip component properties, but not other instanced subobjects
		if (InProperty->HasAnyPropertyFlags(CPF_InstancedReference | CPF_ContainsInstancedReference))
		{
			const FObjectProperty* ObjProp = CastField<const FObjectProperty>(InProperty);
			if (!ObjProp)
			{
				if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(InProperty))
				{
					ObjProp = CastField<const FObjectProperty>(ArrayProp->Inner);
				}
			}

			if (ObjProp && ObjProp->PropertyClass->IsChildOf<UActorComponent>())
			{
				return true;
			}
		}

		return FDataCachePropertyReader::ShouldSkipProperty(InProperty);
	}
};

FDataCacheDuplicatedObjectData::FDataCacheDuplicatedObjectData(UObject* InObject)
	: DuplicatedObject(InObject)
	, ObjectPathDepth(0)
{
	if (DuplicatedObject)
	{
		for (UObject* Outer = DuplicatedObject; Outer; Outer = Outer->GetOuter())
		{
			++ObjectPathDepth;
		}
	}
}

bool FDataCacheDuplicatedObjectData::Serialize(FArchive& Ar)
{
	enum class EVersion : uint8
	{
		InitialVersion = 0,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	EVersion Version = EVersion::LatestVersion;
	Ar << Version;

	if (Version > EVersion::LatestVersion)
	{
		Ar.SetError();
		return true;
	}

	FString ObjectClassPath;
	FString ObjectOuterPath;
	FName ObjectName;
	uint32 ObjectPersistentFlags = 0;
	TArray<uint8> ObjectData;

	if (Ar.IsSaving() && DuplicatedObject)
	{
		UClass* ObjectClass = DuplicatedObject->GetClass();
		ObjectClassPath = ObjectClass ? ObjectClass->GetPathName() : FString();
		ObjectOuterPath = DuplicatedObject->GetOuter() ? DuplicatedObject->GetOuter()->GetPathName() : FString();
		ObjectName = DuplicatedObject->GetFName();
		ObjectPersistentFlags = DuplicatedObject->GetFlags() & RF_Load;
		if (ObjectClass)
		{
			FMemoryWriter MemAr(ObjectData);
			FObjectAndNameAsStringProxyArchive Writer(MemAr, false);
			ObjectClass->SerializeTaggedProperties(Writer, (uint8*)DuplicatedObject, ObjectClass, nullptr);
		}
	}

	Ar << ObjectClassPath;
	Ar << ObjectOuterPath;
	Ar << ObjectName;
	Ar << ObjectPersistentFlags;
	Ar << ObjectData;

	// If loading use the de-serialized property to assign DuplicatedObject
	if (Ar.IsLoading())
	{
		DuplicatedObject = nullptr;
		if (!ObjectClassPath.IsEmpty())
		{
			// Get the object class
			if (UClass* ObjectClass = LoadObject<UClass>(nullptr, *ObjectClassPath))
			{
				// Get the object outer
				if (UObject* FoundOuter = StaticFindObject(UObject::StaticClass(), nullptr, *ObjectOuterPath))
				{
					// Create the duplicated object
					DuplicatedObject = NewObject<UObject>(FoundOuter, ObjectClass, *ObjectName.ToString(), (EObjectFlags)ObjectPersistentFlags);

					// Deserialize the duplicated object
					FMemoryReader MemAr(ObjectData);
					FObjectAndNameAsStringProxyArchive Reader(MemAr, false);
					ObjectClass->SerializeTaggedProperties(Reader, (uint8*)DuplicatedObject, ObjectClass, nullptr);
				}
			}
		}
	}

	return true;
}

FActorInstanceData::FActorInstanceData(const AActor* SourceActor)
	: ActorClass(SourceActor->GetClass())
{
	FActorPropertyWriter PropertyWriter(SourceActor, *this);

	// Cache off the length of an array that will come from SerializeTaggedProperties that had no properties saved in to it.
	auto GetSizeOfEmptyArchive = []() -> int32
	{
		const AActor* DummyActor = GetDefault<AActor>();
		FActorInstanceData DummyInstanceData;
		FDataCachePropertyWriter NullWriter(nullptr, DummyInstanceData);
		UClass* DummyActorClass = DummyActor->GetClass();

		// By serializing the component with itself as its defaults we guarantee that no properties will be written out
		DummyActorClass->SerializeTaggedProperties(NullWriter, (uint8*)DummyActor, DummyActorClass, (uint8*)DummyActor);

		check(DummyInstanceData.GetDuplicatedObjects().Num() == 0 && DummyInstanceData.GetReferencedObjects().Num() == 0);
		return DummyInstanceData.GetSavedProperties().Num();
	};

	static const int32 SizeOfEmptyArchive = GetSizeOfEmptyArchive();

	// SerializeTaggedProperties will always put a sentinel NAME_None at the end of the Archive. 
	// If that is the only thing in the buffer then empty it because we want to know that we haven't stored anything.
	if (SavedProperties.Num() == SizeOfEmptyArchive)
	{
		SavedProperties.Empty();
	}
}

const UClass* FActorInstanceData::GetActorClass() const
{
	return *ActorClass;
}

void FActorInstanceData::ApplyToActor(AActor* Actor, const ECacheApplyPhase CacheApplyPhase)
{
	if ((CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript || CacheApplyPhase == ECacheApplyPhase::NonConstructionScript) && SavedProperties.Num() > 0)
	{
		FActorPropertyReader ActorPropertyReader(Actor, *this);
	}
}


FActorComponentInstanceSourceInfo::FActorComponentInstanceSourceInfo(const UActorComponent* SourceComponent)
	: FActorComponentInstanceSourceInfo(SourceComponent->GetArchetype(), SourceComponent->CreationMethod, SourceComponent->GetUCSSerializationIndex())
{
}

FActorComponentInstanceSourceInfo::FActorComponentInstanceSourceInfo(TObjectPtr<const UObject> InSourceComponentTemplate, EComponentCreationMethod InSourceComponentCreationMethod, int32 InSourceComponentTypeSerializedIndex)
	: SourceComponentTemplate(InSourceComponentTemplate)
	, SourceComponentCreationMethod(InSourceComponentCreationMethod)
	, SourceComponentTypeSerializedIndex(InSourceComponentTypeSerializedIndex)
{
}

bool FActorComponentInstanceSourceInfo::MatchesComponent(const UActorComponent* Component) const
{
	return MatchesComponent(Component, Component->GetArchetype());
}

bool FActorComponentInstanceSourceInfo::MatchesComponent(const UActorComponent* Component, const UObject* ComponentTemplate) const
{
	if (SourceComponentTemplate
		&& SourceComponentTemplate->GetClass() == ComponentTemplate->GetClass()
		&& (SourceComponentTemplate == ComponentTemplate || (GIsReinstancing && SourceComponentTemplate->GetFName() == ComponentTemplate->GetFName()))
		&& SourceComponentCreationMethod == Component->CreationMethod
		)
	{
		if (SourceComponentCreationMethod != EComponentCreationMethod::UserConstructionScript)
		{
			return true;
		}
		
		if (SourceComponentTypeSerializedIndex >= 0)
		{
			check(SourceComponentCreationMethod == EComponentCreationMethod::UserConstructionScript);
			return SourceComponentTypeSerializedIndex == Component->GetUCSSerializationIndex();
		}
	}

	return false;
}

void FActorComponentInstanceSourceInfo::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SourceComponentTemplate);
}


FActorComponentInstanceData::FActorComponentInstanceData()
	: SourceComponentTemplate(nullptr)
	, SourceComponentCreationMethod(EComponentCreationMethod::Native)
	, SourceComponentTypeSerializedIndex(INDEX_NONE)
{}

FActorComponentInstanceData::FActorComponentInstanceData(const UActorComponent* SourceComponent)
{
	check(SourceComponent);
	SourceComponentTemplate = SourceComponent->GetArchetype();
	SourceComponentCreationMethod = SourceComponent->CreationMethod;
	SourceComponentTypeSerializedIndex = SourceComponent->GetUCSSerializationIndex();

	if (SourceComponent->IsEditableWhenInherited())
	{
		FComponentPropertyWriter ComponentPropertyWriter(SourceComponent, *this);

		// Cache off the length of an array that will come from SerializeTaggedProperties that had no properties saved in to it.
		auto GetSizeOfEmptyArchive = []() -> int32
		{
			const UActorComponent* DummyComponent = GetDefault<UActorComponent>();
			FActorComponentInstanceData DummyInstanceData;
			FComponentPropertyWriter NullWriter(nullptr, DummyInstanceData);
			UClass* ComponentClass = DummyComponent->GetClass();
			
			// By serializing the component with itself as its defaults we guarantee that no properties will be written out
			ComponentClass->SerializeTaggedProperties(NullWriter, (uint8*)DummyComponent, ComponentClass, (uint8*)DummyComponent);

			check(DummyInstanceData.GetDuplicatedObjects().Num() == 0 && DummyInstanceData.GetReferencedObjects().Num() == 0);
			return DummyInstanceData.GetSavedProperties().Num();
		};

		static const int32 SizeOfEmptyArchive = GetSizeOfEmptyArchive();

		// SerializeTaggedProperties will always put a sentinel NAME_None at the end of the Archive. 
		// If that is the only thing in the buffer then empty it because we want to know that we haven't stored anything.
		if (SavedProperties.Num() == SizeOfEmptyArchive)
		{
			SavedProperties.Empty();
		}
	}
}

bool FActorComponentInstanceData::MatchesComponent(const UActorComponent* Component, const UObject* ComponentTemplate) const
{
	return FActorComponentInstanceSourceInfo(SourceComponentTemplate, SourceComponentCreationMethod, SourceComponentTypeSerializedIndex).MatchesComponent(Component, ComponentTemplate);
}

void FActorComponentInstanceData::ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase)
{
	// After the user construction script has run we will re-apply all the cached changes that do not conflict
	// with a change that the user construction script made.
	if ((CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript || CacheApplyPhase == ECacheApplyPhase::NonConstructionScript) && SavedProperties.Num() > 0)
	{
		if (CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript)
		{
			Component->DetermineUCSModifiedProperties();
		}
		else if (CacheApplyPhase == ECacheApplyPhase::NonConstructionScript)
		{
			// When this case is used, we want to apply all properties, even UCS modified ones
			Component->ClearUCSModifiedProperties();
		}

		for (const FDataCacheDuplicatedObjectData& DuplicatedObjectData : GetDuplicatedObjects())
		{
			if (DuplicatedObjectData.DuplicatedObject)
			{
				if (UObject* OtherObject = StaticFindObjectFast(nullptr, Component, DuplicatedObjectData.DuplicatedObject->GetFName()))
				{
					OtherObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}
				DuplicatedObjectData.DuplicatedObject->Rename(nullptr, Component, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			}
		}

		FComponentPropertyReader ComponentPropertyReader(Component, *this);

		Component->PostApplyToComponent();
	}
}

UObject* FInstanceCacheDataBase::GetUniqueTransientObject(UClass* ForClass)
{
	if (ForClass->ClassWithin && ForClass->ClassWithin != ForClass && ForClass->ClassWithin != UObject::StaticClass())
	{
		// If this results in a lot of new objects, we could do a GetObjectsWithOuter on the result of
		// GetUniqueTransientObject(ForClass->ClassWithin) and reuse an object.But that doesn't seem likely
		// to be necessary as this is a pretty edge case situation
		FScopedAllowAbstractClassAllocation AllowAbstract;
		return NewObject<UObject>(GetUniqueTransientObject(ForClass->ClassWithin), ForClass->ClassWithin, NAME_None, RF_Transient);
	}
	else if (UniqueTransientPackage.DuplicatedObject == nullptr)
	{
		FScopedAllowAbstractClassAllocation AllowAbstract;
		UniqueTransientPackage = FDataCacheDuplicatedObjectData(NewObject<UObject>(GetTransientPackage()));
	}
	return UniqueTransientPackage.DuplicatedObject;
}

FInstanceCacheDataBase::~FInstanceCacheDataBase() = default;

void FInstanceCacheDataBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ReferencedObjects);
}

void FActorComponentInstanceData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
	Collector.AddReferencedObject(SourceComponentTemplate);
}

FComponentInstanceDataCache::FComponentInstanceDataCache(const AActor* Actor)
{
	if (Actor != nullptr)
	{
		const bool bIsChildActor = Actor->IsChildActor();

		TInlineComponentArray<UActorComponent*> Components(Actor);

		ComponentsInstanceData.Reserve(Components.Num());

		// Grab per-instance data we want to persist
		for (UActorComponent* Component : Components)
		{
			if (bIsChildActor || Component->IsCreatedByConstructionScript()) // Only cache data from 'created by construction script' components
			{
				TStructOnScope<FActorComponentInstanceData> ComponentInstanceData = Component->GetComponentInstanceData();
				if (ComponentInstanceData.IsValid())
				{
					ComponentsInstanceData.Add(MoveTemp(ComponentInstanceData));
				}
			}
			else if (Component->CreationMethod == EComponentCreationMethod::Instance)
			{
				// If the instance component is attached to a BP component we have to be prepared for the possibility that it will be deleted
				if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
				{
					if (SceneComponent->GetAttachParent() && SceneComponent->GetAttachParent()->IsCreatedByConstructionScript())
					{
						// In rare cases the root component can be unset so walk the hierarchy and find what is probably the root component for the purposes of storing off the relative transform
						USceneComponent* RelativeToComponent = Actor->GetRootComponent();
						if (RelativeToComponent == nullptr)
						{
							RelativeToComponent = SceneComponent->GetAttachParent();
							while (RelativeToComponent->GetAttachParent() && RelativeToComponent->GetAttachParent()->GetOwner() == Actor)
							{
								RelativeToComponent = RelativeToComponent->GetAttachParent();
							}
						}

						SceneComponent->ConditionalUpdateComponentToWorld();
						InstanceComponentTransformToRootMap.Add(SceneComponent, SceneComponent->GetComponentTransform().GetRelativeTransform(RelativeToComponent->GetComponentTransform()));
					}
				}
			}
		}
	}
}

void FComponentInstanceDataCache::Serialize(FArchive& Ar)
{
	enum class EVersion : uint8
	{
		InitialVersion = 0,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	EVersion Version = EVersion::LatestVersion;
	Ar << Version;

	if (Version > EVersion::LatestVersion)
	{
		Ar.SetError();
		return;
	}


	if (Ar.IsLoading())
	{
		// Since not all properties are serializable we don't want to deserialize the map directly,
		// so we deserialize it in a temp array and copy it over
		TArray<TStructOnScope<FActorComponentInstanceData>> TempInstanceData;
		Ar << TempInstanceData;
		CopySerializableProperties(MoveTemp(TempInstanceData));
	}
	else
	{
		Ar << ComponentsInstanceData;
	}

	Ar << InstanceComponentTransformToRootMap;
}

void FComponentInstanceDataCache::GetComponentHierarchy(const AActor* Actor, TArray<UActorComponent*, TInlineAllocator<NumInlinedActorComponents>>& OutComponentHierarchy)
{
	// We want to apply instance data from the root node down to ensure changes such as transforms 
	// propagate correctly so we will build the components list in a breadth-first manner.
	OutComponentHierarchy.Reset(Actor->GetComponents().Num());

	auto AddComponentHierarchy = [Actor, &OutComponentHierarchy](USceneComponent* Component)
	{
		int32 FirstProcessIndex = OutComponentHierarchy.Num();

		// Add this to our list and make it our starting node
		OutComponentHierarchy.Add(Component);

		int32 CompsToProcess = 1;

		while (CompsToProcess)
		{
			// track how many elements were here
			const int32 StartingProcessedCount = OutComponentHierarchy.Num();

			// process the currently unprocessed elements
			for (int32 ProcessIndex = 0; ProcessIndex < CompsToProcess; ++ProcessIndex)
			{
				USceneComponent* SceneComponent = CastChecked<USceneComponent>(OutComponentHierarchy[FirstProcessIndex + ProcessIndex]);

				// add all children to the end of the array
				for (int32 ChildIndex = 0; ChildIndex < SceneComponent->GetNumChildrenComponents(); ++ChildIndex)
				{
					if (USceneComponent* ChildComponent = SceneComponent->GetChildComponent(ChildIndex))
					{
						// We don't want to recurse in to child actors (or any other attached actor) when applying the instance cache, 
						// components within a child actor are handled by applying the instance data to the child actor component
						if (ChildComponent->GetOwner() == Actor)
						{
							OutComponentHierarchy.Add(ChildComponent);
						}
					}
				}
			}

			// next loop start with the nodes we just added
			FirstProcessIndex = StartingProcessedCount;
			CompsToProcess = OutComponentHierarchy.Num() - StartingProcessedCount;
		}
	};

	if (USceneComponent* RootComponent = Actor->GetRootComponent())
	{
		AddComponentHierarchy(RootComponent);
	}

	for (UActorComponent* Component : Actor->GetComponents())
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			if (SceneComponent != Actor->GetRootComponent())
			{
				// Scene components that aren't attached to the root component hierarchy won't already have been processed so process them now.
				// * If there is an unattached scene component 
				// * If there is a scene component attached to another Actor's hierarchy
				// * If the scene is not registered (likely because bAutoRegister is false or component is marked pending kill), then we may not have successfully attached to our parent and properly been handled
				USceneComponent* ParentComponent = SceneComponent->GetAttachParent();
				if ((ParentComponent == nullptr)
					|| (ParentComponent->GetOwner() != Actor)
					|| (!SceneComponent->IsRegistered() && !ParentComponent->GetAttachChildren().Contains(SceneComponent)))
				{
					AddComponentHierarchy(SceneComponent);
				}
			}
		}
		else if (Component)
		{
			OutComponentHierarchy.Add(Component);
		}
	}
}

void FComponentInstanceDataCache::ApplyToActor(AActor* Actor, const ECacheApplyPhase CacheApplyPhase) const
{
	if (Actor != nullptr)
	{
		TInlineComponentArray<UActorComponent*> Components;
		GetComponentHierarchy(Actor, Components);

		// Apply per-instance data.
		TBitArray<> ComponentInstanceDataToConsider(true, ComponentsInstanceData.Num());
		for (int32 Index = 0; Index < ComponentsInstanceData.Num(); ++Index)
		{
			if (!ComponentsInstanceData[Index].IsValid())
			{
				ComponentInstanceDataToConsider[Index] = false;
			}
		}

		const bool bIsChildActor = Actor->IsChildActor();
		for (UActorComponent* ComponentInstance : Components)
		{
			if (ComponentInstance && (bIsChildActor || ComponentInstance->IsCreatedByConstructionScript())) // Only try and apply data to 'created by construction script' components
			{
				// Cache template here to avoid redundant calls in the loop below
				const UObject* ComponentTemplate = ComponentInstance->GetArchetype();

				for (TConstSetBitIterator<> ComponentInstanceDataIt(ComponentInstanceDataToConsider); ComponentInstanceDataIt; ++ComponentInstanceDataIt)
				{
					const TStructOnScope<FActorComponentInstanceData>& ComponentInstanceData = ComponentsInstanceData[ComponentInstanceDataIt.GetIndex()];
					if (ComponentInstanceData->MatchesComponent(ComponentInstance, ComponentTemplate))
					{
						ComponentInstanceData->ApplyToComponent(ComponentInstance, CacheApplyPhase);
						ComponentInstanceDataToConsider[ComponentInstanceDataIt.GetIndex()] = false;
						break;
					}
				}
			}
		}

		// Once we're done attaching, if we have any unattached instance components move them to the root
		for (const auto& InstanceTransformPair : InstanceComponentTransformToRootMap)
		{
			check(Actor->GetRootComponent());

			USceneComponent* SceneComponent = InstanceTransformPair.Key;
			if (SceneComponent && !IsValid(SceneComponent))
			{
				SceneComponent->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
				SceneComponent->SetRelativeTransform(InstanceTransformPair.Value);
			}
		}
	}
}

void FComponentInstanceDataCache::FindAndReplaceInstances(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	for (TStructOnScope<FActorComponentInstanceData>& ComponentInstanceData : ComponentsInstanceData)
	{
		if (ComponentInstanceData.IsValid())
		{
			ComponentInstanceData->FindAndReplaceInstances(OldToNewInstanceMap);
		}
	}
	TArray<USceneComponent*> SceneComponents;
	ObjectPtrDecay(InstanceComponentTransformToRootMap).GenerateKeyArray(SceneComponents);

	for (USceneComponent* SceneComponent : SceneComponents)
	{
		if (UObject* const* NewSceneComponent = OldToNewInstanceMap.Find(SceneComponent))
		{
			if (*NewSceneComponent)
			{
				InstanceComponentTransformToRootMap.Add(CastChecked<USceneComponent>(*NewSceneComponent), InstanceComponentTransformToRootMap.FindAndRemoveChecked(SceneComponent));
			}
			else
			{
				InstanceComponentTransformToRootMap.Remove(SceneComponent);
			}
		}
	}
}

void FComponentInstanceDataCache::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(InstanceComponentTransformToRootMap);

	for (TStructOnScope<FActorComponentInstanceData>& ComponentInstanceData : ComponentsInstanceData)
	{
		if (ComponentInstanceData.IsValid())
		{
			ComponentInstanceData->AddReferencedObjects(Collector);
		}
	}
}

void FComponentInstanceDataCache::CopySerializableProperties(TArray<TStructOnScope<FActorComponentInstanceData>> InComponentsInstanceData)
{
	auto CopyProperties = [](TStructOnScope<FActorComponentInstanceData>& DestData, const TStructOnScope<FActorComponentInstanceData>& SrcData)
	{
		for (TFieldIterator<const FProperty> PropertyIt(SrcData.GetStruct(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::IncludeDeprecated, EFieldIteratorFlags::ExcludeInterfaces); PropertyIt; ++PropertyIt)
		{
			const void* SrcValuePtr = PropertyIt->ContainerPtrToValuePtr<void>(SrcData.Get());
			void* DestValuePtr = PropertyIt->ContainerPtrToValuePtr<void>(DestData.Get());
			PropertyIt->CopyCompleteValue(DestValuePtr, SrcValuePtr);
		}
	};

	for (TStructOnScope<FActorComponentInstanceData>& InstanceData : InComponentsInstanceData)
	{
		TStructOnScope<FActorComponentInstanceData>* DestInstanceData = ComponentsInstanceData.FindByPredicate([&InstanceData](TStructOnScope<FActorComponentInstanceData>& InInstanceData)
		{
			return InstanceData->GetComponentTemplate() == InInstanceData->GetComponentTemplate() && InstanceData.GetStruct() == InInstanceData.GetStruct();
		});
		// if we find a ComponentInstanceData to apply it too, we copy the properties over
		if (DestInstanceData)
		{
			CopyProperties(*DestInstanceData, InstanceData);
		}
		// otherwise we just add our to the list, since no component instance data was created for it
		else
		{
			ComponentsInstanceData.Add(MoveTemp(InstanceData));
		}
	}
}

