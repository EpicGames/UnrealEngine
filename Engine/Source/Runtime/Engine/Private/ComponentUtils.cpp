// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentUtils.h"
#include "Engine/SCS_Node.h"

namespace ComponentUtils
{

	/**
	 * A helper for retrieving the simple-construction-script that this component belongs in.
	 *
	 * @param  Component	The component you want the SCS for.
	 * @return The component's blueprint SCS (NULL if one wasn't found).
	 */
	USimpleConstructionScript const* GetSimpleConstructionScript(USceneComponent const* Component)
	{
		USimpleConstructionScript const* BlueprintSCS = NULL;

		check(Component);
		UObject const* ComponentOuter = Component->GetOuter();

		if(UBlueprint const* const OuterBlueprint = Cast<UBlueprint const>(ComponentOuter))
		{
			BlueprintSCS = OuterBlueprint->SimpleConstructionScript;
		}
		else if(UBlueprintGeneratedClass const* const GeneratedClass = Cast<UBlueprintGeneratedClass>(ComponentOuter))
		{
			BlueprintSCS = GeneratedClass->SimpleConstructionScript;
		}

		return BlueprintSCS;
	}

	USCS_Node* FindCorrespondingSCSNode(USceneComponent const* ComponentObj)
	{
		if (USimpleConstructionScript const* BlueprintSCS = GetSimpleConstructionScript(ComponentObj))
		{
			for (USCS_Node* SCSNode : BlueprintSCS->GetAllNodes())
			{
				if (SCSNode && SCSNode->ComponentTemplate == ComponentObj)
				{
					return SCSNode;
				}
			}
		}

		return nullptr;
	}

	/**
	 * A static helper function used to retrieve a component's scene parent
	 *
	 * @param  SceneComponentObject	The component you want the attached parent for
	 * @return A pointer to the component's scene parent (NULL if there was not one)
	 */
	USceneComponent* GetAttachedParent(USceneComponent const* SceneComponentObject)
	{
		USceneComponent* SceneParent = SceneComponentObject->GetAttachParent();
		if(SceneParent == nullptr)
		{
			USCS_Node* const SCSNode = FindCorrespondingSCSNode(SceneComponentObject);
			// if we didn't find a corresponding simple-construction-script node
			if(SCSNode == nullptr)
			{
				return nullptr;
			}

			USimpleConstructionScript const* BlueprintSCS = GetSimpleConstructionScript(SceneComponentObject);
			check(BlueprintSCS != nullptr);

			USCS_Node* const ParentSCSNode = BlueprintSCS->FindParentNode(SCSNode);
			if(ParentSCSNode != nullptr)
			{
				SceneParent = Cast<USceneComponent>(ParentSCSNode->ComponentTemplate);
			}
		}

		return SceneParent;
	}
}
