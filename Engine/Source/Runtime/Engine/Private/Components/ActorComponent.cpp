// Copyright Epic Games, Inc. All Rights Reserved.
// ActorComponent.cpp: Actor component implementation.

#include "Components/ActorComponent.h"

#include "AI/NavigationSystemBase.h"
#include "Async/ParallelFor.h"
#include "ComponentRecreateRenderStateContext.h"
#include "ComponentReregisterContext.h"
#include "Components/PrimitiveComponent.h"
#include "ComponentUtils.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Engine/ActorChannel.h"
#include "Engine/AssetUserData.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/InputDelegateBinding.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingPersistent.h"
#include "Engine/MemberReference.h"
#include "Engine/NetDriver.h"
#include "Engine/SimpleConstructionScript.h"
#include "EngineStats.h"
#include "GameFramework/InputSettings.h"
#include "HAL/LowLevelMemStats.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/UObjectToken.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"
#include "ObjectTrace.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/BodySetup.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "SceneInterface.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/FrameworkObjectVersion.h"
#include "PSOPrecacheMaterial.h"
#include "PSOPrecacheSettings.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "ObjectCacheContext.h"
#include "AutoRTFM.h"

#if WITH_EDITOR
#include "Kismet2/ComponentEditorUtils.h"
#include "ObjectCacheEventSink.h"
#include "StaticMeshCompiler.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#endif

#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/UObjectMigrationContext.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorComponent)

#define LOCTEXT_NAMESPACE "ActorComponent"

DEFINE_LOG_CATEGORY(LogActorComponent);

DECLARE_CYCLE_STAT(TEXT("RegisterComponent"), STAT_RegisterComponent, STATGROUP_Component);
DECLARE_CYCLE_STAT(TEXT("UnregisterComponent"), STAT_UnregisterComponent, STATGROUP_Component);

DECLARE_CYCLE_STAT(TEXT("Component OnRegister"), STAT_ComponentOnRegister, STATGROUP_Component);
DECLARE_CYCLE_STAT(TEXT("Component OnUnregister"), STAT_ComponentOnUnregister, STATGROUP_Component);

DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Component CreateRenderState"), STAT_ComponentCreateRenderState, STATGROUP_Component, EStatFlags::Verbose);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Component DestroyRenderState"), STAT_ComponentDestroyRenderState, STATGROUP_Component, EStatFlags::Verbose);

DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Component CreatePhysicsState"), STAT_ComponentCreatePhysicsState, STATGROUP_Component, EStatFlags::Verbose);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Component DestroyPhysicsState"), STAT_ComponentDestroyPhysicsState, STATGROUP_Component, EStatFlags::Verbose);

// Should we tick latent actions fired for a component at the same time as the component?
// - Non-zero values behave the same way as actors do, ticking pending latent action when the component ticks, instead of later on in the frame
// - Prior to 4.16, components behaved as if the value were 0, which meant their latent actions behaved differently to actors
//UE_DEPRECATED(4.16, "This CVar will be removed, with the behavior permanently changing in the future to always tick component latent actions along with the component")
int32 GTickComponentLatentActionsWithTheComponent = 1;

// Should we tick latent actions fired for a component at the same time as the component?
FAutoConsoleVariableRef GTickComponentLatentActionsWithTheComponentCVar(
	TEXT("t.TickComponentLatentActionsWithTheComponent"),
	GTickComponentLatentActionsWithTheComponent,
	TEXT("Should we tick latent actions fired for a component at the same time as the component?\n")
	TEXT(" 0: Tick component latent actions later on in the frame (behavior prior to 4.16, provided for games relying on the old behavior but will be removed in the future)\n")
	TEXT(" 1: Tick component latent actions at the same time as the component (default)"));

/** Enable to log out all render state create, destroy and updatetransform events */
#define LOG_RENDER_STATE 0

/** Static delegate broadcasted when the first active FGlobalComponentReregisterContext is created. */
FGlobalComponentReregisterContext::FOnGlobalComponentReregisterContextCreated FGlobalComponentReregisterContext::GlobalComponentReregisterContextCreatedEvent;

/** Static delegate broadcasted when the last active FGlobalComponentReregisterContext is destroyed. */
FGlobalComponentReregisterContext::FOnGlobalComponentReregisterContextDestroyed FGlobalComponentReregisterContext::GlobalComponentReregisterContextDestroyedEvent;

/** Static var indicating activity of reregister context */
int32 FGlobalComponentReregisterContext::ActiveGlobalReregisterContextCount = 0;

/** Static var indicating activity of recreate render state context */
int32 FGlobalComponentRecreateRenderStateContext::ActiveGlobalRecreateRenderStateContextCount = 0;


bool GDefaultUseSubObjectReplicationList = false;
static FAutoConsoleVariableRef CVarDefaultUseSubObjectReplicationList(
	TEXT("net.SubObjects.DefaultUseSubObjectReplicationList"),
	GDefaultUseSubObjectReplicationList,
	TEXT("Do actors and actorcomponents replicate subobjects using the registration method by default."));

// Allows for CreatePhysicsState to be deferred, to batch work and parallelize.
int32 GEnableDeferredPhysicsCreation = 0;
FAutoConsoleVariableRef CVarEnableDeferredPhysicsCreation(
	TEXT("p.EnableDeferredPhysicsCreation"), 
	GEnableDeferredPhysicsCreation,
	TEXT("Enables/Disables deferred physics creation.")
);

// Allows for CreatePhysicsState to be deferred, to batch work and parallelize.
int32 GPrecachePSOsOnComponentRecreateRenderContext = 1;
FAutoConsoleVariableRef CVarPrecachePSOsOnComponentRecreateRenderContext(
	TEXT("r.PrecachePSOsOnComponentRecreateRenderContext"),
	GPrecachePSOsOnComponentRecreateRenderContext,
	TEXT("If > 0, re-creating a component's render context will also re-precache the component's PSOs.")
);

FRegisterComponentContext::FRegisterComponentContext(UWorld* InWorld)
	: World(InWorld)
	, Timeout(UE::FTimeout::AlwaysExpired())
	, IncrementalUpdateAddPrimitiveBatchSize(0)
	, IncrementalUpdateNumComponentsToUpdate(0)
{}

FRegisterComponentContext::~FRegisterComponentContext()
{
	Process();
}

void FRegisterComponentContext::AddPrimitive(UPrimitiveComponent* PrimitiveComponent)
{
	checkSlow(!AddPrimitiveBatches.Contains(PrimitiveComponent));
	AddPrimitiveBatches.Add(PrimitiveComponent);
	if ((IncrementalUpdateAddPrimitiveBatchSize > 0) && (AddPrimitiveBatches.Num() >= IncrementalUpdateAddPrimitiveBatchSize))
	{
		Process();
	}
}

void FRegisterComponentContext::AddSendRenderDynamicData(UPrimitiveComponent* PrimitiveComponent)
{
	checkSlow(!SendRenderDynamicDataPrimitives.Contains(PrimitiveComponent));
	SendRenderDynamicDataPrimitives.Add(PrimitiveComponent);
}

void FRegisterComponentContext::SendRenderDynamicData(FRegisterComponentContext* Context, UPrimitiveComponent* PrimitiveComponent)
{
	if (Context)
	{
		Context->AddSendRenderDynamicData(PrimitiveComponent);
	}
	else
	{
		PrimitiveComponent->SendRenderDynamicData_Concurrent();
	}
}

FRegisterComponentContext& FRegisterComponentContext::SetIncrementalUpdateTimeout(const UE::FTimeout& InTimeout)
{
	Timeout = InTimeout;
	return *this;
}

FRegisterComponentContext& FRegisterComponentContext::SetIncrementalUpdateAddPrimitiveBatchSize(int32 Value)
{
	IncrementalUpdateAddPrimitiveBatchSize = Value;
	return *this;
}

FRegisterComponentContext& FRegisterComponentContext::SetIncrementalUpdateNumComponentsToUpdate(int32 Value)
{
	IncrementalUpdateNumComponentsToUpdate = Value;
	return *this;
}

FRegisterComponentContext& FRegisterComponentContext::SetAsyncRegisterLevelContext(FAsyncRegisterLevelContext* Value)
{
	return *this;
}

bool FRegisterComponentContext::OnIncrementalRegisterComponentsDone()
{
	// We need to process pending adds prior to rerunning the construction scripts, which may internally
	// perform removals / adds themselves.
	Process();
	return true;
}

void FRegisterComponentContext::Process()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRegisterComponentContext::Process)

	if (AddPrimitiveBatches.Num())
	{
		FSceneInterface* Scene = World->Scene;
		const bool bAppCanEverRender = FApp::CanEverRender();

		bool bSingleThreaded = !FApp::ShouldUseThreadingForPerformance();
#if WITH_EDITOR
		// This is required for async static mesh compilation in case a scene proxy is not async aware.
		// A stall until the compilation is finished might occur, and this is only supported on the game thread for now.
		bSingleThreaded |= FStaticMeshCompilingManager::Get().IsAsyncStaticMeshCompilationEnabled();
#endif
		ParallelFor(AddPrimitiveBatches.Num(),
			[&](int32 Index)
			{
				FTaskTagScope Scope(ETaskTag::EParallelGameThread);
				UPrimitiveComponent* Component = AddPrimitiveBatches[Index];

				// AActor::PostRegisterAllComponents (called by AActor::IncrementalRegisterComponents) can trigger code 
				// that either unregisters or re-registers components. If unregistered, skip this component.
				// If re-registered, FRegisterComponentContext is not passed, so SceneProxy can be created.
				if (IsValid(Component) && Component->IsRegistered())
				{
					if (Component->IsRenderStateCreated() || !bAppCanEverRender)
					{
						// Skip if SceneProxy is already created
						if (Component->SceneProxy == nullptr)
						{
							Scene->AddPrimitive(Component);
						}
					}
					else // Fallback for some edge case where the component renderstate are missing
					{
						Component->CreateRenderState_Concurrent(nullptr);
					}
				}
			},
			bSingleThreaded
		);
		AddPrimitiveBatches.Empty();
	}

	for (UPrimitiveComponent* Primitive : SendRenderDynamicDataPrimitives)
	{
		// With incremetal updates the component can be registered, added to send render data queue, then destroyed by an actor chain so we must test it's still valid before sending render data.
		if (::IsValid(Primitive))
		{
			Primitive->SendRenderDynamicData_Concurrent();
		}
	}
	SendRenderDynamicDataPrimitives.Empty();

	check(AddPrimitiveBatches.IsEmpty());
	check(SendRenderDynamicDataPrimitives.IsEmpty());
}

void UpdateAllPrimitiveSceneInfosForSingleComponent(UActorComponent* InComponent, TSet<FSceneInterface*>* InScenesToUpdateAllPrimitiveSceneInfosForBatching /* = nullptr*/)
{
	if (FSceneInterface* Scene = InComponent->GetScene())
	{
		if (InScenesToUpdateAllPrimitiveSceneInfosForBatching == nullptr)
		{
			UE::RenderCommandPipe::FSyncScope SyncScope;

			// If no batching is available (this ComponentReregisterContext is not created by a FGlobalComponentReregisterContext), issue one update per component
			ENQUEUE_RENDER_COMMAND(UpdateAllPrimitiveSceneInfosCmd)([Scene](FRHICommandListImmediate& RHICmdList) {
				Scene->UpdateAllPrimitiveSceneInfos(RHICmdList);
			});
		}
		else
		{
			// Try to batch the updates inside FGlobalComponentReregisterContext
			InScenesToUpdateAllPrimitiveSceneInfosForBatching->Add(Scene);
		}
	}
}

void UpdateAllPrimitiveSceneInfosForSingleComponentInterface (IPrimitiveComponent* InComponentInterface, TSet<FSceneInterface*>* InScenesToUpdateAllPrimitiveSceneInfosForBatching /* = nullptr*/)
{
	if (FSceneInterface* Scene = InComponentInterface->GetScene())
	{
		if (InScenesToUpdateAllPrimitiveSceneInfosForBatching == nullptr)
		{
			UE::RenderCommandPipe::FSyncScope SyncScope;

			// If no batching is available (this ComponentReregisterContext is not created by a FGlobalComponentReregisterContext), issue one update per component
			ENQUEUE_RENDER_COMMAND(UpdateAllPrimitiveSceneInfosCmd)([Scene](FRHICommandListImmediate& RHICmdList) {
				Scene->UpdateAllPrimitiveSceneInfos(RHICmdList);
			});
		}
		else
		{
			// Try to batch the updates inside FGlobalComponentReregisterContext
			InScenesToUpdateAllPrimitiveSceneInfosForBatching->Add(Scene);
		}
	}
}

void UpdateAllPrimitiveSceneInfosForScenes(TSet<FSceneInterface*> ScenesToUpdateAllPrimitiveSceneInfos)
{
	if (ScenesToUpdateAllPrimitiveSceneInfos.Num())
	{
		UE::RenderCommandPipe::FSyncScope SyncScope;

		ENQUEUE_RENDER_COMMAND(UpdateAllPrimitiveSceneInfosCmd)(
			[ScenesToUpdateAllPrimitiveSceneInfos](FRHICommandListImmediate& RHICmdList)
			{
				for (FSceneInterface* Scene : ScenesToUpdateAllPrimitiveSceneInfos)
				{
					Scene->UpdateAllPrimitiveSceneInfos(RHICmdList);
				}
			}
		);
	}
}

FGlobalComponentReregisterContext::FGlobalComponentReregisterContext()
{
	ActiveGlobalReregisterContextCount++;

	// wait until resources are released
	FlushRenderingCommands();

	// Detach all actor components.
	for(UActorComponent* Component : TObjectRange<UActorComponent>())
	{
		ComponentContexts.Add(new FComponentReregisterContext(Component, &ScenesToUpdateAllPrimitiveSceneInfos));
	}

	// Only broadcast if this is the first active context
	if (ActiveGlobalReregisterContextCount == 1)
	{
		GlobalComponentReregisterContextCreatedEvent.Broadcast();
	}

	UpdateAllPrimitiveSceneInfos();
}

FGlobalComponentReregisterContext::FGlobalComponentReregisterContext(const TArray<UClass*>& ExcludeComponents)
{
	// Check if this is the first active context
	if (++ActiveGlobalReregisterContextCount == 1)
	{
		// wait until resources are released
		FlushRenderingCommands();
		
		// Detach only actor components that are not in the excluded list
		for (UActorComponent* Component : TObjectRange<UActorComponent>())
		{
			bool bShouldReregister=true;
			for (UClass* ExcludeClass : ExcludeComponents)
			{
				if( ExcludeClass &&
					Component->IsA(ExcludeClass) )
				{
					bShouldReregister = false;
					break;
				}
			}
			if( bShouldReregister )
			{
				ComponentContexts.Add(new FComponentReregisterContext(Component, &ScenesToUpdateAllPrimitiveSceneInfos));
			}
		}

		GlobalComponentReregisterContextCreatedEvent.Broadcast();
		
		UpdateAllPrimitiveSceneInfos();
	}
}

FGlobalComponentReregisterContext::~FGlobalComponentReregisterContext()
{
	check(ActiveGlobalReregisterContextCount > 0);

	// Check if this is the last active context
	if (--ActiveGlobalReregisterContextCount == 0)
	{
		ComponentContexts.Empty();

		GlobalComponentReregisterContextDestroyedEvent.Broadcast();
		
		UpdateAllPrimitiveSceneInfos();
	}
}

void FGlobalComponentReregisterContext::UpdateAllPrimitiveSceneInfos()
{
	UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(ScenesToUpdateAllPrimitiveSceneInfos));

	check(ScenesToUpdateAllPrimitiveSceneInfos.Num() == 0);
}


FComponentRecreateRenderStateContext::~FComponentRecreateRenderStateContext()
{
	if (Component && !Component->IsRenderStateCreated() && Component->IsRegistered())
	{
		if (GPrecachePSOsOnComponentRecreateRenderContext)
		{
			Component->PrecachePSOs();
		}
		Component->CreateRenderState_Concurrent(nullptr);

		UpdateAllPrimitiveSceneInfosForSingleComponent(Component, ScenesToUpdateAllPrimitiveSceneInfos);
	}

	if (ComponentInterface && !ComponentInterface->IsRenderStateCreated() && ComponentInterface->IsRegistered())
	{
		if (GPrecachePSOsOnComponentRecreateRenderContext)
		{
 			ComponentInterface->PrecachePSOs();
		}
		ComponentInterface->CreateRenderState(nullptr);

		UpdateAllPrimitiveSceneInfosForSingleComponentInterface(ComponentInterface, ScenesToUpdateAllPrimitiveSceneInfos);
	}
}

FGlobalComponentRecreateRenderStateContext::FGlobalComponentRecreateRenderStateContext()
{
	if (FApp::CanEverRender())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGlobalComponentRecreateRenderStateContext::FGlobalComponentRecreateRenderStateContext);

		//ActiveGlobalRecreateRenderStateContextCount++;

		// wait until resources are released
		FlushRenderingCommands();

		FObjectCacheContextScope ObjectCacheScope;
		for (IPrimitiveComponent* PrimitiveComponent: ObjectCacheScope.GetContext().GetPrimitiveComponents())
		{
			if (PrimitiveComponent->IsRegistered() && PrimitiveComponent->IsRenderStateCreated())
			{
				ComponentContexts.Emplace(PrimitiveComponent, &ScenesToUpdateAllPrimitiveSceneInfos);
			}
		}

		// recreate render state for all components.
		for (UActorComponent* Component : TObjectRange<UActorComponent>())
		{
			// Those are obtained through FObjectCacheContext
			if (!Component->IsA<UPrimitiveComponent>())
			{
				if (Component->IsRegistered() && Component->IsRenderStateCreated())
				{
					ComponentContexts.Emplace(Component, &ScenesToUpdateAllPrimitiveSceneInfos);
				}
			}
		}

		UpdateAllPrimitiveSceneInfos();
	}
}

FGlobalComponentRecreateRenderStateContext::FGlobalComponentRecreateRenderStateContext(const TArray<UActorComponent*>& InComponents)
{
	if (FApp::CanEverRender() /*&& ++ActiveGlobalRecreateRenderStateContextCount == 1*/)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGlobalComponentRecreateRenderStateContext::FGlobalComponentRecreateRenderStateContext);

		// wait until resources are released
		FlushRenderingCommands();

		// recreate render state for provided components.
		for (UActorComponent* Component : InComponents)
		{
			if (Component->IsRegistered() && Component->IsRenderStateCreated())
			{
				ComponentContexts.Emplace(Component, &ScenesToUpdateAllPrimitiveSceneInfos);
			}
		}

		UpdateAllPrimitiveSceneInfos();
	}
}

FGlobalComponentRecreateRenderStateContext::~FGlobalComponentRecreateRenderStateContext()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGlobalComponentRecreateRenderStateContext::~FGlobalComponentRecreateRenderStateContext);

	if (FApp::CanEverRender())
	{
		//check(ActiveGlobalRecreateRenderStateContextCount > 0);

		// Check if this is the last active context
		//if (--ActiveGlobalRecreateRenderStateContextCount == 0)
		{
			// Clear the PSO material request cache to make sure PSO collection happens again on possible changed data
			if (GPrecachePSOsOnComponentRecreateRenderContext)
			{
				ClearMaterialPSORequests();
				UMaterial* PSOPrecacheFallbackMaterial = UPSOPrecacheSettingsManager::GetFallbackMaterial();
				if (PSOPrecacheFallbackMaterial)
				{
					UMaterialInterface::PrecacheFallbackMaterialPSOs(PSOPrecacheFallbackMaterial);
				}
			}

			ComponentContexts.Empty();

			UpdateAllPrimitiveSceneInfos();
		}
	}
}

void FGlobalComponentRecreateRenderStateContext::UpdateAllPrimitiveSceneInfos()
{
	UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(ScenesToUpdateAllPrimitiveSceneInfos));

	check(ScenesToUpdateAllPrimitiveSceneInfos.Num() == 0);
}

// Create Physics global delegate
FActorComponentGlobalCreatePhysicsSignature UActorComponent::GlobalCreatePhysicsDelegate;
// Destroy Physics global delegate
FActorComponentGlobalDestroyPhysicsSignature UActorComponent::GlobalDestroyPhysicsDelegate;
// Render state dirty global delegate
UActorComponent::FOnMarkRenderStateDirty UActorComponent::MarkRenderStateDirtyEvent;

const FString UActorComponent::ComponentTemplateNameSuffix(TEXT("_GEN_VARIABLE"));
TMap<UActorComponent*, TArray<FSimpleMemberReference>> UActorComponent::AllUCSModifiedProperties;
FTransactionallySafeRWLock UActorComponent::AllUCSModifiedPropertiesLock;

UActorComponent::UActorComponent(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
	: Super(ObjectInitializer)
	, RegistrationState(EComponentRegistrationState::None)
	, PhysicsStateAsyncCreationScene(nullptr)
	, PhysicsStateAsyncCreationState(EPhysicsStateAsyncCreationState::None)
{
	// OwnerPrivate must be valid until destruction, even after aborting.
	UE_AUTORTFM_OPEN_NO_VALIDATION
	{
		OwnerPrivate = GetTypedOuter<AActor>();
	};

	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.bCanEverTick = false;
	PrimaryComponentTick.bAllowTickBatching = true;
	PrimaryComponentTick.bRunTransactionally = true;
	PrimaryComponentTick.SetTickFunctionEnable(false);

	MarkedForEndOfFrameUpdateState = 0;
	MarkedForEndOfFrameUpdateArrayIndex = INDEX_NONE;
	UCSSerializationIndex = INDEX_NONE;

	CreationMethod = EComponentCreationMethod::Native;

	bAllowReregistration = true;
	bAutoRegister = true;
	bNetAddressable = false;
	bEditableWhenInherited = true;
#if WITH_EDITOR
	bCanUseCachedOwner = true;
#endif

	bCanEverAffectNavigation = false;
	bNavigationRelevant = false;

	bMarkedForPreEndOfFrameSync = false;
	bReadyForEarlyEndOfFrameUpdate = false;
	bRenderStateUpdating = false;
	bAsyncPhysicsTickEnabled = false;
	bCallAsyncPhysicsStateCreatedDelegate = false;

	bReplicateUsingRegisteredSubObjectList = GDefaultUseSubObjectReplicationList;
}

void UActorComponent::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsInGameThread())
	{
		UEngineElementsLibrary::CreateEditorComponentElement(this);
		using namespace UE::Editor::DataStorage;
		if (ICompatibilityProvider* Storage = UE::Editor::DataStorage::GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
		{
			Storage->AddCompatibleObject(this);
		}
	}
#endif	// WITH_EDITOR

	// Instance components will be added during the owner's initialization
	if (OwnerPrivate && CreationMethod != EComponentCreationMethod::Instance)
	{
		OwnerPrivate->AddOwnedComponent(this);
	}
}

void UActorComponent::PostLoad()
{
	Super::PostLoad();
	   
#if WITH_EDITORONLY_DATA
	if (GetLinkerUEVersion() < VER_UE4_ACTOR_COMPONENT_CREATION_METHOD)
	{
		if (IsTemplate())
		{
			CreationMethod = EComponentCreationMethod::Native;
		}
		else if (bCreatedByConstructionScript_DEPRECATED)
		{
			CreationMethod = EComponentCreationMethod::SimpleConstructionScript;
		}
		else if (bInstanceComponent_DEPRECATED)
		{
			CreationMethod = EComponentCreationMethod::Instance;
		}

		if (CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
		{
			UBlueprintGeneratedClass* Class = CastChecked<UBlueprintGeneratedClass>(GetOuter()->GetClass());
			while (Class)
			{
				USimpleConstructionScript* SCS = Class->SimpleConstructionScript;
				if (SCS != nullptr && SCS->FindSCSNode(GetFName()))
				{
					break;
				}
				else
				{
					Class = Cast<UBlueprintGeneratedClass>(Class->GetSuperClass());
					if (Class == nullptr)
					{
						CreationMethod = EComponentCreationMethod::UserConstructionScript;
					}
				}
			}
		}
	}
#endif

	if (CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
	{
		if ((GetLinkerUEVersion() < VER_UE4_TRACK_UCS_MODIFIED_PROPERTIES) && !HasAnyFlags(RF_ClassDefaultObject))
		{
			DetermineUCSModifiedProperties();
		}
#if WITH_EDITORONLY_DATA
		else if (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::ActorComponentUCSModifiedPropertiesSparseStorage)
		{
			if (UCSModifiedProperties_DEPRECATED.Num())
			{
				UE::TRWScopeLock Lock(AllUCSModifiedPropertiesLock, SLT_Write);
				AllUCSModifiedProperties.Add(this, MoveTemp(UCSModifiedProperties_DEPRECATED));
			}
		}
#endif
	}
	else
	{
#if WITH_EDITORONLY_DATA
		// For a brief period of time we were inadvertently storing these for all components, need to clear it out
		UCSModifiedProperties_DEPRECATED.Empty();

		if (CreationMethod == EComponentCreationMethod::UserConstructionScript)
		{
			if (GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::StoringUCSSerializationIndex)
			{
				bNeedsUCSSerializationIndexEvaluted = true;
			}
		}
#endif
	}

#if WITH_EDITOR
	if (bMarkPendingKillOnPostLoad)
	{
		MarkAsGarbage();
		bMarkPendingKillOnPostLoad = false;
	}
#endif // WITH_EDITOR
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
void UActorComponent::PostMigrate(const struct FUObjectMigrationContext& MigrationContext)
{
	Super::PostMigrate(MigrationContext);

	// We want to be able to store "annotations" (metadata) about this Component while we loan it out, so that
	// when we have the loan returned to us, we can check what has changed...
	struct FLoanedState
	{
		bool bWasRegistered = false;
		bool bWasDestroyed = false;
		bool bWasPhysicsStateCreated = false;
		bool bWasTickFunctionRegistered = false;
		bool bWasComponentTickEnabled = false;

		bool operator==(const FLoanedState&) const = default;
		bool IsDefault() const
		{
			static const FLoanedState DefaultLoanedState;
			return *this == DefaultLoanedState;
		}
	};
	static FUObjectAnnotationSparse<FLoanedState,/*bAutoRemove=*/true> RemoteObjects_LoanedStates;
	FLoanedState NewState {
		.bWasRegistered = !!bRegistered, .bWasDestroyed = IsBeingDestroyed(), .bWasPhysicsStateCreated = !!bPhysicsStateCreated,
		.bWasTickFunctionRegistered = !!bTickFunctionsRegistered, .bWasComponentTickEnabled = IsComponentTickEnabled()
	};

	if (MigrationContext.MigrationSide == EObjectMigrationSide::Send)
	{
		const EResidence Residence = UE::RemoteObject::Handle::GetResidence(this);
		const EObjectMigrationSendType SendType = MigrationContext.GetObjectMigrationSendType(this);
		UE_LOGF(LogActorComponent, VeryVerbose, "%s: %ls %ls (Residence: %ls)", __func__, ToString(SendType), *GetName(), EnumToString(Residence));

		if (Residence != EResidence::Local)
		{
			ensureMsgf(Residence == EResidence::LocalNotReady, TEXT("Trying to send a non-local object"));
			return;
		}

		// Remove the actor and it's sub-objects from the network id cache since these objects are now considered remote. It's assumed that the
		// network id cache will borrow any objects that are not in the cache but are considered remote.
		//
		// Loan is an exception because the server is still retaining ownership of this object and can still be dereferenced.
		if (SendType != EObjectMigrationSendType::Loan)
		{
			GetWorld()->GetNetDriver()->RemoveObjectFromNetworkIdCache(this);
		}

		// On the next receive, force components to re-register their replicated subobjects when the owning actor calls ResetOwnedComponents.
		bIsReadyForReplication = false;

		// If we're loaning, we're keeping ownership; in all other cases we either don't own, or won't own the object after it's migrated
		const bool bKeepOwnership = (SendType == EObjectMigrationSendType::Loan);
		if (bKeepOwnership)
		{
			checkfSlow(!RemoteObjects_LoanedStates.GetAnnotationMap().Find(this), TEXT("%hs: Expected no existing Loaned State for %s (it should have been consumed when received, or not loaned out twice in a row)"), __func__, *GetName());
			RemoteObjects_LoanedStates.AddAnnotation(this, NewState);
		}
		else
		{
			// If we're transferring ownership, make sure we don't keep anything registered locally
			if (IsRegistered())
			{
				UnregisterComponent();
			}

			// Prevent tick functions from happening this frame (unregistering just prevents it from happening *next* frame)
			SetComponentTickEnabled(false);
		}

		if (SendType == EObjectMigrationSendType::ReassignOwnership)
		{
			// It's not necessary to explicitly defer a call to StopReplicatingActorComponent(). It will automatically be
			// deferred by the deferred call to StopReplicatingActor() for the parent actor.
			//
			// NOTE: This should be re-enabled if components ever migrate separately to actors
			//
			//UE::Net::FDeferredReplicationSystemCalls::StopReplicatingActorComponentDefer(this, EEndReplicationFlags::Migrated);
		}

		// Prevent calling StopReplicatingActorComponent() in EndPlay(). 
		bCallStopReplicationInEndPlay = false;
	}
	else if (ensureMsgf(MigrationContext.MigrationSide == EObjectMigrationSide::Receive, TEXT("Invalid MigrationContext: Side was not Send or Receive")))
	{
		const EObjectMigrationRecvType RecvType = MigrationContext.GetObjectMigrationRecvType(this);
		UE_LOGF(LogActorComponent, VeryVerbose, "%s: %ls %ls", __func__, ToString(RecvType), *GetName());

		// It's not necessary to explicitly defer a call to StartReplicatingActorComponent(). It will automatically be
		// deferred by the deferred call to StartReplicatingActor() for the parent actor.
		//
		// NOTE: This should be re-enabled if components ever migrate separately to actors
		// 
		//if (RecvType == EObjectMigrationRecvType::AssignedOwnership)
		//{
		//	UE::Net::FDeferredReplicationSystemCalls::StartReplicatingActorComponentDefer(GetOwner(), this);
		//}

		// Add the actor and it's sub-objects to the network id cache since these objects are no longer considered remote and can be derferenced
		// through the network id cache (e.g. by RPC argument and object deserialization).
		if (HasBegunPlay() && IsSupportedForNetworking() && (RecvType != EObjectMigrationRecvType::ReturnedLoan))
		{
			GetWorld()->GetNetDriver()->AddObjectToNetworkIdCache(this);
		}

		// We are expecting this flow from AActor, but now we must do this per-component in such a way
		// that we are not mutating our previously deserialized data.
		// RegisterAllComponents();
		// PreInitializeComponents();
		// InitializeComponents();
		// PostInitializeComponents();

		if (RecvType == EObjectMigrationRecvType::ReturnedLoan)
		{
			const FLoanedState LoanedState = RemoteObjects_LoanedStates.GetAndRemoveAnnotation(this);

			const AActor* Owner = GetOwner();
			if (ensure(Owner) && Owner->IsActorBeingDestroyed())
			{
				// If the Actor is being destroyed, then this Component would have been destroyed through
				// the Actor's Destroy call, so let's ignore the state changes
			}
			else if (NewState != LoanedState)
			{
				const bool bNewlyDestroyed = IsBeingDestroyed() && !LoanedState.bWasDestroyed;
				UE_CLOGF(bNewlyDestroyed, LogActorComponent, Warning, "%s: Destroying %ls back on owning server (it was destroyed while loaned-out)", __func__, *GetName());

				// Reset these variables to how they were when we loaned them out, and then go through the code path locally
				bIsBeingDestroyed = LoanedState.bWasDestroyed;
				bRegistered = LoanedState.bWasRegistered;
				bPhysicsStateCreated = LoanedState.bWasPhysicsStateCreated;
				bTickFunctionsRegistered = LoanedState.bWasTickFunctionRegistered;

				// These are a chain of possibilities, with destroying covering all other cases
				if (NewState.bWasDestroyed != LoanedState.bWasDestroyed)
				{
					DestroyComponent();
				}
				else if (NewState.bWasRegistered != LoanedState.bWasRegistered)
				{
					if (NewState.bWasRegistered)
					{
						RegisterComponent();
					}
					else
					{
						UnregisterComponent();
					}
				}
				else // These are all individually set states (not chained, like the above are)
				{
					if (NewState.bWasPhysicsStateCreated != LoanedState.bWasPhysicsStateCreated)
					{
						if (NewState.bWasPhysicsStateCreated)
						{
							CreatePhysicsState();
						}
						else
						{
							DestroyPhysicsState();
						}
					}

					if (NewState.bWasTickFunctionRegistered != LoanedState.bWasTickFunctionRegistered)
					{
						RegisterAllComponentTickFunctions(NewState.bWasTickFunctionRegistered);
					}
				}

				// Since the tick can already be scheduled, we can enable/disable this regardless of the above states
				if (NewState.bWasComponentTickEnabled != LoanedState.bWasComponentTickEnabled)
				{
					SetComponentTickEnabled(NewState.bWasComponentTickEnabled);
				}

				FLoanedState AppliedState {
					.bWasRegistered = !!bRegistered, .bWasDestroyed = IsBeingDestroyed(), .bWasPhysicsStateCreated = !!bPhysicsStateCreated,
					.bWasTickFunctionRegistered = !!bTickFunctionsRegistered, .bWasComponentTickEnabled = IsComponentTickEnabled()
				};

				ensureMsgf(AppliedState == NewState, TEXT("%hs: ReturnedLoan State Application code path was not correct for %s"), __func__, *GetName());
			}
		}
		else // if (RecvType != EObjectMigrationRecvType::ReturnedLoan)
		{
			// If we did not previously have this Component, we need to register it (RegisterAllComponents)
			// However, since that causes a bunch of side effects which have already happened at some point
			// let's skip right ahead to OnRegister() in a state that causes the least amount of side-effects:
			if (IsRegistered())
			{
				bool bRestoreAutoActivate = bAutoActivate;

				WorldPrivate = GetOwner()->GetWorld();
				RegistrationState = EComponentRegistrationState::PreRegistered;
				bAutoActivate = false;
				bRegistered = false;
				{
					OnRegister();
				}
				if (bPhysicsStateCreated)
				{
					bPhysicsStateCreated = false;
					OnCreatePhysicsState();
				}
				bAutoActivate = bRestoreAutoActivate;
			}
		}

		if (RecvType == EObjectMigrationRecvType::AssignedOwnership)
		{
			// If we're receiving ownership, we are now in charge of its Tick
			if (bTickFunctionsRegistered)
			{
				bTickFunctionsRegistered = false;
				RegisterAllComponentTickFunctions(true);
			}

			SetComponentTickEnabled(bComponentTickEnabled);

			// We now own this object, so if we EndPlay, then actually stop replication.
			bCallStopReplicationInEndPlay = true;
		}
	}
}
#endif

bool UActorComponent::Rename( const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags )
{
	bRoutedPostRename = false;

	const FName OldName = GetFName();
	const UObject* OldOuter = GetOuter();
	
	const bool bRenameSuccessful = Super::Rename(InName, NewOuter, Flags);
	
	const bool bMoved = (OldName != GetFName()) || (OldOuter != GetOuter());
	if (!bRoutedPostRename && ((Flags & REN_Test) == 0) && bMoved)
	{
		UE_LOGF(LogActorComponent, Fatal, "%ls failed to route PostRename.  Please call Super::PostRename() in your <className>::PostRename() function. ", *GetFullName() );
	}

	return bRenameSuccessful;
}

void UActorComponent::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	if (OldOuter != GetOuter())
	{
		OwnerPrivate = GetTypedOuter<AActor>();
		AActor* OldOwner = (OldOuter->IsA<AActor>() ? static_cast<AActor*>(OldOuter) : OldOuter->GetTypedOuter<AActor>());

		if (OwnerPrivate != OldOwner)
		{
			if (OldOwner)
			{
				OldOwner->RemoveOwnedComponent(this);
			}
			if (OwnerPrivate)
			{
				OwnerPrivate->AddOwnedComponent(this);
			}

#if WITH_EDITOR
			using namespace UE::Editor::DataStorage;
			if (ICoreProvider* Storage = UE::Editor::DataStorage::GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				RowHandle ComponentRow = Storage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(this));
				Storage->AddColumns<FTypedElementSyncFromWorldTag>(ComponentRow);
			}
#endif

			TArray<UObject*> Children;
			GetObjectsWithOuter(this, Children, EGetObjectsFlags::None);

			for (int32 Index = 0; Index < Children.Num(); ++Index)
			{
				UObject* Child = Children[Index];

				// Cut off if we have a nested Actor
				if (!Child->IsA<AActor>())
				{
					if (UActorComponent* ChildComponent = Cast<UActorComponent>(Child))
					{
						ChildComponent->OwnerPrivate = OwnerPrivate;
						if (OldOwner)
						{
							OldOwner->RemoveOwnedComponent(ChildComponent);
						}
						if (OwnerPrivate)
						{
							OwnerPrivate->AddOwnedComponent(ChildComponent);
						}
					}
					GetObjectsWithOuter(Child, Children, EGetObjectsFlags::None);
				}
			}
		}
	}

	bRoutedPostRename = true;
}

bool UActorComponent::IsCreatedByConstructionScript() const
{
	return ((CreationMethod == EComponentCreationMethod::SimpleConstructionScript) || (CreationMethod == EComponentCreationMethod::UserConstructionScript));
}

#if WITH_EDITORONLY_DATA
void UActorComponent::DetermineUCSSerializationIndexForLegacyComponent()
{
	check(bNeedsUCSSerializationIndexEvaluted);
	bNeedsUCSSerializationIndexEvaluted = false;

	int32 ComputedSerializationIndex = INDEX_NONE;

	if (CreationMethod == EComponentCreationMethod::UserConstructionScript)
	{
		if (AActor* ComponentOwner = GetOwner())
		{
			if (ComponentOwner->BlueprintCreatedComponents.Num() > 0)
			{
				UObject* ComponentTemplate = GetArchetype();

				bool bFound = false;
				for (const UActorComponent* BlueprintCreatedComponent : ComponentOwner->BlueprintCreatedComponents)
				{
					if (BlueprintCreatedComponent && BlueprintCreatedComponent->CreationMethod == EComponentCreationMethod::UserConstructionScript)
					{
						if (BlueprintCreatedComponent == this)
						{
							++ComputedSerializationIndex;
							bFound = true;
							break;
						}
						else if (BlueprintCreatedComponent->GetArchetype() == ComponentTemplate)
						{
							++ComputedSerializationIndex;
						}
					}
				}
				if (!bFound)
				{
					ComputedSerializationIndex = INDEX_NONE;
				}
			}
		}
	}
	UCSSerializationIndex = ComputedSerializationIndex;
}
#endif

#if WITH_EDITOR
TUniquePtr<FWorldPartitionComponentDesc> UActorComponent::CreateClassComponentDesc() const
{
	return nullptr;
}

TUniquePtr<FWorldPartitionComponentDesc> UActorComponent::CreateComponentDesc() const
{
	TUniquePtr<FWorldPartitionComponentDesc> ComponentDesc(CreateClassComponentDesc());

	if (ComponentDesc.IsValid())
	{
		ComponentDesc->Init(this);
	}

	return ComponentDesc;
}

TUniquePtr<class FWorldPartitionComponentDesc> UActorComponent::StaticCreateClassComponentDesc(const TSubclassOf<UActorComponent>& ComponentClass)
{
	return CastChecked<UActorComponent>(ComponentClass->GetDefaultObject())->CreateClassComponentDesc();
}

void UActorComponent::CheckForErrors()
{
	if (AActor* MyOwner = GetOwner())
	{
		if (GetClass()->HasAnyClassFlags(CLASS_Deprecated))
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ComponentName"), FText::FromString(GetName()));
			Arguments.Add(TEXT("OwnerName"), FText::FromString(MyOwner->GetName()));
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(MyOwner))
				->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_DeprecatedClass", "{ComponentName}::{OwnerName} is obsolete and must be removed (Class is deprecated)" ), Arguments ) ) )
				->AddToken(FMapErrorToken::Create(FMapErrors::DeprecatedClass));
		}

		if (GetClass()->HasAnyClassFlags(CLASS_Abstract))
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ComponentName"), FText::FromString(GetName()));
			Arguments.Add(TEXT("OwnerName"), FText::FromString(MyOwner->GetName()));
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(MyOwner))
				->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_AbstractClass", "{ComponentName}::{OwnerName} is obsolete and must be removed (Class is abstract)" ), Arguments ) ))
				->AddToken(FMapErrorToken::Create(FMapErrors::AbstractClass));
		}
	}
}
#endif

bool UActorComponent::IsOwnerSelected() const
{
	AActor* MyOwner = GetOwner();
	return MyOwner && MyOwner->IsSelected();
}

UWorld* UActorComponent::GetWorld_Uncached() const
{
	UWorld* ComponentWorld = nullptr;

	AActor* MyOwner = GetOwner();
	// If we don't have a world yet, it may be because we haven't gotten registered yet, but we can try to look at our owner
	if (MyOwner && !MyOwner->HasAnyFlags(RF_ClassDefaultObject))
	{
		ComponentWorld = MyOwner->GetWorld();
	}

	if( ComponentWorld == nullptr )
	{
		// As a fallback check the outer of this component for a world. In some cases components are spawned directly in the world
		ComponentWorld = Cast<UWorld>(GetOuter());
	}

	return ComponentWorld;
}

bool UActorComponent::ComponentHasTag(FName Tag) const
{
	return (Tag != NAME_None) && ComponentTags.Contains(Tag);
}


ENetMode UActorComponent::InternalGetNetMode() const
{
	AActor* MyOwner = GetOwner();
	return MyOwner ? MyOwner->GetNetMode() : NM_Standalone;
}

FSceneInterface* UActorComponent::GetScene() const
{
	return (WorldPrivate ? WorldPrivate->Scene : NULL);
}

FPhysScene* UActorComponent::GetPhysicsScene() const
{
	return WorldPrivate ? WorldPrivate->GetPhysicsScene() : nullptr;
}

ULevel* UActorComponent::GetComponentLevel() const
{
	// For model components Level is outer object
	AActor* MyOwner = GetOwner();
	return (MyOwner ? MyOwner->GetLevel() : GetTypedOuter<ULevel>());
}

bool UActorComponent::ComponentIsInLevel(const ULevel *TestLevel) const
{
	return (GetComponentLevel() == TestLevel);
}

bool UActorComponent::ComponentIsInPersistentLevel(bool bIncludeLevelStreamingPersistent) const
{
	ULevel* MyLevel = GetComponentLevel();
	UWorld* MyWorld = GetWorld();

	if (MyLevel == NULL || MyWorld == NULL)
	{
		return false;
	}

	return ( (MyLevel == MyWorld->PersistentLevel) || ( bIncludeLevelStreamingPersistent && MyWorld->GetStreamingLevels().Num() > 0 &&
														Cast<ULevelStreamingPersistent>(MyWorld->GetStreamingLevels()[0]) &&
														MyWorld->GetStreamingLevels()[0]->GetLoadedLevel() == MyLevel ) );
}

FString UActorComponent::GetReadableName() const
{
	const AActor* Owner = GetOwner();
	FString Result = (Owner ? Owner->GetActorNameOrLabel() : TEXT("None")) + TEXT(".") + GetName();
	UObject const *Add = AdditionalStatObject();
	if (Add)
	{
		Result += TEXT(" ");
		Add->AppendName(Result);
	}
	return Result;
}

void UActorComponent::BeginDestroy()
{
	if (bHasBegunPlay)
	{
		EndPlay(EEndPlayReason::Destroyed);
	}

	// Ensure that we call UninitializeComponent before we destroy this component
	if (bHasBeenInitialized)
	{
		UninitializeComponent();
	}

	bIsReadyForReplication = false;

	ExecuteUnregisterEvents();

	// Ensure that we call OnComponentDestroyed before we destroy this component
	if (bHasBeenCreated)
	{
		OnComponentDestroyed(GExitPurge);
	}

	WorldPrivate = nullptr;

	// Remove from the parent's OwnedComponents list
	if (AActor* MyOwner = GetOwner())
	{
		MyOwner->RemoveOwnedComponent(this);
	}

#if WITH_EDITOR
	using namespace UE::Editor::DataStorage;
	if (ICompatibilityProvider* Storage = UE::Editor::DataStorage::GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		Storage->RemoveCompatibleObject(this);
	}
	UEngineElementsLibrary::DestroyEditorComponentElement(this);
#endif	// WITH_EDITOR

	ClearUCSModifiedProperties();

	Super::BeginDestroy();
}

bool UActorComponent::NeedsLoadForClient() const
{
	check(GetOuter());
	// For Component Blueprints, avoid calling into the class to avoid recursion
	bool bNeedsLoadOuter = HasAnyFlags(RF_ClassDefaultObject) || GetOuter()->NeedsLoadForClient();
	return (!IsEditorOnly() && bNeedsLoadOuter && Super::NeedsLoadForClient());
}

bool UActorComponent::NeedsLoadForServer() const
{
	check(GetOuter());
	// For Component Blueprints, avoid calling into the class to avoid recursion
	bool bNeedsLoadOuter = HasAnyFlags(RF_ClassDefaultObject) || GetOuter()->NeedsLoadForServer();
	return (!IsEditorOnly() && bNeedsLoadOuter && Super::NeedsLoadForServer());
}

bool UActorComponent::NeedsLoadForEditorGame() const
{
	return !IsEditorOnly() && Super::NeedsLoadForEditorGame();
}

int32 UActorComponent::GetFunctionCallspace( UFunction* Function, FFrame* Stack )
{
	if ((Function->FunctionFlags & FUNC_Static))
	{
		// Try to use the same logic as function libraries for static functions, will try to use the global context to check authority only/cosmetic
		return GEngine->GetGlobalFunctionCallspace(Function, this, Stack);
	}

	AActor* MyOwner = GetOwner();
	return (MyOwner ? MyOwner->GetFunctionCallspace(Function, Stack) : FunctionCallspace::Local);
}

bool UActorComponent::CallRemoteFunction( UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack )
{
	bool bProcessed = false;

	if (AActor* MyOwner = GetOwner())
	{
		FWorldContext* const Context = GEngine->GetWorldContextFromWorld(GetWorld());
		if (Context != nullptr)
		{
			for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
			{
				if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateFunction(MyOwner, Function))
				{
					Driver.NetDriver->ProcessRemoteFunction(MyOwner, Function, Parameters, OutParms, Stack, this);
					bProcessed = true;
				}
			}
		}
	}

	return bProcessed;
}

#if WITH_EDITOR

/** FComponentReregisterContexts for components which have had PreEditChange called but not PostEditChange. */
static TMap<TWeakObjectPtr<UActorComponent>,FComponentReregisterContext*> EditReregisterContexts;

bool UActorComponent::Modify( bool bAlwaysMarkDirty/*=true*/ )
{
	AActor* MyOwner = GetOwner();
    
	// Components in transient actors should never mark the package as dirty
	bAlwaysMarkDirty = bAlwaysMarkDirty && (!MyOwner || !MyOwner->HasAnyFlags(RF_Transient));

	// If this is a construction script component we don't store them in the transaction buffer.  Instead, mark
	// the Actor as modified so that we store of the transaction annotation that has the component properties stashed
	if (MyOwner)
	{
		if (IsCreatedByConstructionScript() || (UChildActorComponent::ExperimentalAllowPerInstanceChildActorProperties() && MyOwner->IsChildActor()))
		{
			return MyOwner->Modify(bAlwaysMarkDirty);
		}
	}

	return Super::Modify(bAlwaysMarkDirty);
}

ENGINE_API bool GFlushRenderingCommandsOnPreEditChange = true;

void UActorComponent::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if(IsRegistered())
	{
		// The component or its outer could be pending kill when calling PreEditChange when applying a transaction.
		// Don't do do a full recreate in this situation, and instead simply detach.
		if(IsValidChecked(this))
		{
			// One way this check can fail is that component subclass does not call Super::PostEditChangeProperty
			checkf(!EditReregisterContexts.Find(this),
				TEXT("UActorComponent::PreEditChange(this=%s, owner actor class=%s) already had PreEditChange called on it with no matching PostEditChange; You might be missing a call to Super::PostEditChangeProperty in your PostEditChangeProperty implementation"),
				*GetFullNameSafe(this),
				(GetOwner() != nullptr) ? *GetOwner()->GetClass()->GetName() : TEXT("no owner"));

			EditReregisterContexts.Add(this,new FComponentReregisterContext(this));
		}
		else
		{
			ExecuteUnregisterEvents();
			WorldPrivate = nullptr;
		}
	}
	// Flush rendering commands to ensure the rendering thread processes the component detachment before it is modified.
	if (GFlushRenderingCommandsOnPreEditChange)
	{
		FlushRenderingCommands();
	}
}

void UActorComponent::PreEditUndo()
{
	Super::PreEditUndo();

	OwnerPrivate = nullptr;
	bCanUseCachedOwner = false;
}

void UActorComponent::PostEditUndo()
{
	// Objects marked pending kill don't call PostEditChange() from UObject::PostEditUndo(),
	// so they can leave an EditReregisterContexts entry around if they are deleted by an undo action.
	if(!IsValidChecked(this))
	{
		// For the redo case, ensure that we're no longer in the OwnedComponents array.
		if (AActor* OwningActor = GetOwner())
		{
			OwningActor->RemoveOwnedComponent(this);
		}

		// The reregister context won't bother attaching components that are 'pending kill'. 
		FComponentReregisterContext* ReregisterContext = nullptr;
		if (EditReregisterContexts.RemoveAndCopyValue(this, ReregisterContext))
		{
			delete ReregisterContext;
		}
		else
		{
			// This means there are likely some stale elements left in there now, strip them out
			for (auto It(EditReregisterContexts.CreateIterator()); It; ++It)
			{
				if (!It.Key().IsValid())
				{
					It.RemoveCurrent();
				}
			}
		}
	}
	else
	{
		bIsBeingDestroyed = false;

		OwnerPrivate = GetTypedOuter<AActor>();
		bCanUseCachedOwner = true;

		// Let the component be properly registered, after it was restored.
		if (OwnerPrivate)
		{
			OwnerPrivate->AddOwnedComponent(this);
		}

		TArray<UObject*> Children;
		GetObjectsWithOuter(this, Children, EGetObjectsFlags::None);

		for (int32 Index = 0; Index < Children.Num(); ++Index)
		{
			UObject* Child = Children[Index];

			// Cut off if we have a nested Actor
			if (!Child->IsA<AActor>())
			{
				if (UActorComponent* ChildComponent = Cast<UActorComponent>(Child))
				{
					if (ChildComponent->OwnerPrivate)
					{
						ChildComponent->OwnerPrivate->RemoveOwnedComponent(ChildComponent);
					}
					ChildComponent->OwnerPrivate = OwnerPrivate;
					if (OwnerPrivate)
					{
						OwnerPrivate->AddOwnedComponent(ChildComponent);
					}
				}
				GetObjectsWithOuter(Child, Children, EGetObjectsFlags::None);
			}
		}

		if (UWorld* MyWorld = GetWorld())
		{
			MyWorld->UpdateActorComponentEndOfFrameUpdateState(this);
		}
	}
	Super::PostEditUndo();
}

bool UActorComponent::IsSelectedInEditor() const
{
	return IsValidChecked(this) && GIsComponentSelectedInEditor && GIsComponentSelectedInEditor(this);
}

void UActorComponent::ConsolidatedPostEditChange(const FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_CanEverAffectNavigation = GET_MEMBER_NAME_CHECKED(UActorComponent, bCanEverAffectNavigation);

	FComponentReregisterContext* ReregisterContext = nullptr;
	if(EditReregisterContexts.RemoveAndCopyValue(this, ReregisterContext))
	{
		delete ReregisterContext;

		AActor* MyOwner = GetOwner();
		if ( MyOwner && !MyOwner->IsTemplate() && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive )
		{
			MyOwner->RerunConstructionScripts();
		}
	}
	else
	{
		// This means there are likely some stale elements left in there now, strip them out
		for (auto It(EditReregisterContexts.CreateIterator()); It; ++It)
		{
			if (!It.Key().IsValid())
			{
				It.RemoveCurrent();
			}
		}
	}

	// The component or its outer could be pending kill when calling PostEditChange when applying a transaction.
	// Don't do a full recreate in this situation, and instead simply detach.
	if (!IsValidChecked(this))
	{
		// @todo james should this call UnregisterComponent instead to remove itself from the RegisteredComponents array on the owner?
		ExecuteUnregisterEvents();
		WorldPrivate = nullptr;
	}
	else
	{
		// Navigation specific fix until UE-252220 is fixed
		// Only process navigation status changes on valid components (i.e., RerunConstructionScripts will destroy this component)
		if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.Property->GetFName() == NAME_CanEverAffectNavigation)
		{
			HandleCanEverAffectNavigationChange(/*bForce=*/true);
		}
	}

	for (UAssetUserData* Datum : AssetUserData)
	{
		if (Datum != nullptr)
		{
			Datum->PostEditChangeOwner(PropertyChangedEvent);
		}
	}
	for (UAssetUserData* Datum : AssetUserDataEditorOnly)
	{
		if (Datum != nullptr)
		{
			Datum->PostEditChangeOwner(PropertyChangedEvent);
		}
	}
}

void UActorComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	ConsolidatedPostEditChange(PropertyChangedEvent);
}

void UActorComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	ConsolidatedPostEditChange(PropertyChangedEvent);
}


#endif // WITH_EDITOR

bool UActorComponent::ShouldAsyncCreatePhysicsState(UWorld* WorldContext) const
{
	check(WorldContext);
	check(FPhysScene::SupportsAsyncPhysicsStateCreation());
	TGuardValue<EComponentRegistrationState> Guard(const_cast<UActorComponent*>(this)->RegistrationState, EComponentRegistrationState::PreRegistering);
	return (!bPhysicsStateCreated && WorldContext->GetPhysicsScene() && AllowsAsyncPhysicsStateCreation() && ShouldCreatePhysicsState());
}

bool UActorComponent::ShouldAsyncDestroyPhysicsState() const
{
	return bPhysicsStateCreated && WorldPrivate && GetPhysicsScene() && AllowsAsyncPhysicsStateDestruction();
}

bool UActorComponent::ShouldIncrementalPreRegister(UWorld* WorldContext) const
{
	// For now the only criteria is if the component should create its physics state asynchronously
	return ShouldAsyncCreatePhysicsState(WorldContext);
}

bool UActorComponent::ShouldIncrementalPreUnregister() const
{
	// For now the only criteria is if the component should destroy its physics state asynchronously
	return ShouldAsyncDestroyPhysicsState();
}

void UActorComponent::OnPreRegister()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorComponent::OnPreRegister);
	checkf(WorldPrivate, TEXT("OnPreRegister: %s to %s"), *GetDetailedInfo(), *GetFullNameSafe(GetOwner()));
	checkf(RegistrationState == EComponentRegistrationState::None, TEXT("OnPreRegister: %s to %s"), *GetDetailedInfo(), *GetFullNameSafe(GetOwner()));
	checkf(!bRegistered, TEXT("OnPreRegister: %s to %s"), *GetDetailedInfo(), *GetFullNameSafe(GetOwner()));

	RegistrationState = EComponentRegistrationState::PreRegistering;

	AActor* MyOwner = GetOwner();
	if (ULevel* Level = MyOwner ? MyOwner->GetLevel() : nullptr)
	{
		FLevelRegistrationAccessor::OnPreRegisterComponent(Level, this);
	}

	UpdateComponentToWorld();

	if (ShouldAsyncCreatePhysicsState(WorldPrivate))
	{
		FPhysScene_Chaos* PhysScene = GetPhysicsScene();
		check(PhysScene);
		check(!IsAsyncCreatePhysicsStateRunning());
		check(!IsAsyncDestroyPhysicsStateRunning());
		verify(PhysScene->PushAsyncCreatePhysicsStateJob(this));
		PhysicsStateAsyncCreationState = EPhysicsStateAsyncCreationState::Creating;
		PhysicsStateAsyncCreationScene = PhysScene;
		check(IsAsyncCreatePhysicsStateRunning());
	}
	else
	{
		OnPreRegistered();
	}
}

void UActorComponent::OnPreRegistered()
{
	check(IsPreRegistering());
	RegistrationState = EComponentRegistrationState::PreRegistered;

	AActor* MyOwner = GetOwner();
	if (ULevel* Level = MyOwner ? MyOwner->GetLevel() : nullptr)
	{
		FLevelRegistrationAccessor::OnPreRegisteredComponent(Level, this);
	}
}

void UActorComponent::OnRegister()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorComponent::OnRegister);
#if !UE_BUILD_SHIPPING
	// These are removed in shipping because they are still likely to fail in Test and Development builds, and checks in shipping makes this rather expensive.
	checkf(!IsUnreachable(), TEXT("%s"), *GetDetailedInfo());
	checkf(!GetOuter()->IsTemplate(), TEXT("'%s' (%s)"), *GetOuter()->GetFullName(), *GetDetailedInfo());
	checkf(!IsTemplate(), TEXT("'%s' (%s)"), *GetOuter()->GetFullName(), *GetDetailedInfo() );
	checkf(IsValidChecked(this), TEXT("OnRegister: %s to %s"), *GetDetailedInfo(), GetOwner() ? *GetOwner()->GetFullName() : TEXT("*** No Owner ***") );
#endif
	checkf(WorldPrivate, TEXT("OnRegister: %s to %s"), *GetDetailedInfo(), GetOwner() ? *GetOwner()->GetFullName() : TEXT("*** No Owner ***") );
	checkf(!bRegistered, TEXT("OnRegister: %s to %s"), *GetDetailedInfo(), GetOwner() ? *GetOwner()->GetFullName() : TEXT("*** No Owner ***") );

	const bool bHasBeenPreRegistered = IsPreRegistered();
	bRegistered = true;
	
	// Reset pre-register state as the component is now marked registered
	RegistrationState = EComponentRegistrationState::None;

	// No need to call UpdateComponentToWorld if pre-registered as it has already been called in UActorComponent::OnPreRegister
	if (!bHasBeenPreRegistered)
	{
		UpdateComponentToWorld();
	}

	if (bAutoActivate)
	{
		AActor* Owner = GetOwner();
		if (!WorldPrivate->IsGameWorld() || Owner == nullptr || Owner->IsActorInitialized())
		{
			Activate(true);
		}
	}
}

void UActorComponent::OnUnregister()
{
	check(bRegistered);
	bRegistered = false;

	check(!IsPreRegistering());
	check(!IsPreUnregistering());
	RegistrationState = EComponentRegistrationState::None;

	ClearNeedEndOfFrameUpdate();
}

void UActorComponent::InitializeComponent()
{
	check(bRegistered);
	check(!bHasBeenInitialized);

	bHasBeenInitialized = true;
}

void UActorComponent::UninitializeComponent()
{
	check(bHasBeenInitialized);

	bHasBeenInitialized = false;
}

void UActorComponent::ReadyForReplication()
{
	bIsReadyForReplication = true;
}

void UActorComponent::BeginPlay()
{
	TRACE_OBJECT_LIFETIME_BEGIN(this);

	check(bRegistered);
	check(!bHasBegunPlay);
	checkSlow(bTickFunctionsRegistered); // If this fails, someone called BeginPlay() without first calling RegisterAllComponentTickFunctions().

	if (GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint) || !GetClass()->HasAnyClassFlags(CLASS_Native))
	{
		ReceiveBeginPlay();
	}

	bHasBegunPlay = true;
}

void UActorComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	TRACE_OBJECT_LIFETIME_END(this);

	check(bHasBegunPlay);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (bCallStopReplicationInEndPlay)
#endif
	{
		if (EndPlayReason != EEndPlayReason::EndPlayInEditor && EndPlayReason != EEndPlayReason::Quit)
		{
			UE::Net::FReplicationSystemUtil::StopReplicatingActorComponent(this);
		}
	}

	// If we're in the process of being garbage collected it is unsafe to call out to blueprints
	if (!HasAnyFlags(RF_BeginDestroyed) && !IsUnreachable() && (GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint) || !GetClass()->HasAnyClassFlags(CLASS_Native)))
	{
		ReceiveEndPlay(EndPlayReason);
	}

	bIsReadyForReplication = false;
	bHasBegunPlay = false;
}

TStructOnScope<FActorComponentInstanceData> UActorComponent::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData>(this);
}

void FActorComponentTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FActorComponentTickFunction::ExecuteTick);

	ExecuteTickHelper(Target, Target->bTickInEditor, DeltaTime, TickType, [this, TickType](float DilatedTime)
	{
		Target->TickComponent(DilatedTime, TickType, this);
	});

#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_AUTORTFM_OPEN
	{
		CachedDiagnosticContext = DiagnosticContext(true);
		CachedDiagnosticMessage = DiagnosticMessage();
	};
#endif
}

FString FActorComponentTickFunction::DiagnosticMessage()
{
#if	UE_WITH_REMOTE_OBJECT_HANDLE
	return (Target.GetResidence() != EResidence::Remote ? Target->GetFullName() : Target.GetRemoteId().ToString()) + TEXT("[TickComponent]");
#else
	return Target->GetFullName() + TEXT("[TickComponent]");
#endif
}

FName FActorComponentTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		AActor* OwningActor = Target->GetOwner();
		FString OwnerClassName = OwningActor ? OwningActor->GetClass()->GetName() : TEXT("None");
		// Format is "ComponentClass/OwningActorClass/ComponentName"
		FString ContextString = FString::Printf(TEXT("%s/%s/%s"), *Target->GetClass()->GetName(), *OwnerClassName, *Target->GetName());
		return FName(*ContextString);
	}
	else
	{
		return Target->GetClass()->GetFName();
	}
}


bool UActorComponent::SetupActorComponentTickFunction(struct FTickFunction* TickFunction)
{
	if(TickFunction->bCanEverTick && !IsTemplate())
	{
		AActor* MyOwner = GetOwner();
		if (!MyOwner || !MyOwner->IsTemplate())
		{
			ULevel* ComponentLevel = (MyOwner ? MyOwner->GetLevel() : ToRawPtr(GetWorld()->PersistentLevel));
			TickFunction->SetTickFunctionEnable(TickFunction->bStartWithTickEnabled || TickFunction->IsTickFunctionEnabled());
			TickFunction->RegisterTickFunction(ComponentLevel);
			return true;
		}
	}
	return false;
}

void UActorComponent::SetComponentTickEnabled(bool bEnabled)
{
	if (PrimaryComponentTick.bCanEverTick && !IsTemplate())
	{
		PrimaryComponentTick.SetTickFunctionEnable(bEnabled);
	}
}

void UActorComponent::SetComponentTickEnabledAsync(bool bEnabled)
{
	if (PrimaryComponentTick.bCanEverTick && !IsTemplate())
	{
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.SetComponentTickEnabledAsync"),
			STAT_FSimpleDelegateGraphTask_SetComponentTickEnabledAsync,
			STATGROUP_TaskGraphTasks);

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateUObject(this, &UActorComponent::SetComponentTickEnabled, bEnabled),
			GET_STATID(STAT_FSimpleDelegateGraphTask_SetComponentTickEnabledAsync), NULL, ENamedThreads::GameThread
		);
	}
}

bool UActorComponent::IsComponentTickEnabled() const
{
	return PrimaryComponentTick.IsTickFunctionEnabled();
}

void UActorComponent::SetComponentTickInterval(float TickInterval)
{
	PrimaryComponentTick.TickInterval = TickInterval;
}

void UActorComponent::SetComponentTickIntervalAndCooldown(float TickInterval)
{
	PrimaryComponentTick.UpdateTickIntervalAndCoolDown(TickInterval);
}

float UActorComponent::GetComponentTickInterval() const
{
	return PrimaryComponentTick.TickInterval;
}

static UActorComponent* GTestRegisterComponentTickFunctions = NULL;

void UActorComponent::RegisterComponentTickFunctions(bool bRegister)
{
	if(bRegister)
	{
		if (SetupActorComponentTickFunction(&PrimaryComponentTick))
		{
			PrimaryComponentTick.Target = this;
		}
	}
	else
	{
		if(PrimaryComponentTick.IsTickFunctionRegistered())
		{
			PrimaryComponentTick.UnRegisterTickFunction();
		}
	}

	GTestRegisterComponentTickFunctions = this; // we will verify the super call chain is intact. Don't not copy paste this to a derived class!
}

void UActorComponent::RegisterAllComponentTickFunctions(bool bRegister)
{
	check(GTestRegisterComponentTickFunctions == NULL);
	// Components don't have tick functions until they are registered with the world
	if (bRegistered)
	{
		// Prevent repeated redundant attempts
		if (bTickFunctionsRegistered != bRegister)
		{
			RegisterComponentTickFunctions(bRegister);
			bTickFunctionsRegistered = bRegister;
			checkf(GTestRegisterComponentTickFunctions == this, TEXT("Failed to route component RegisterTickFunctions (%s)"), *GetFullName());
			GTestRegisterComponentTickFunctions = NULL;
		}

		if (bAsyncPhysicsTickEnabled)
		{
			RegisterAsyncPhysicsTickEnabled(bRegister);
		}
	}
}

void UActorComponent::RegisterAsyncPhysicsTickEnabled(bool bRegister)
{
	if (FPhysScene_Chaos* Scene = static_cast<FPhysScene_Chaos*>(GetPhysicsScene()))
	{
		if (bRegister)
		{
			Scene->RegisterAsyncPhysicsTickComponent(this);
		}
		else
		{
			Scene->UnregisterAsyncPhysicsTickComponent(this);
		}
	}
}

void UActorComponent::SetAsyncPhysicsTickEnabled(bool bEnable)
{
	// Components don't have async physics functions until they are registered with the world
	if(bRegistered)
	{
		RegisterAsyncPhysicsTickEnabled(bEnable);
	}
	
	bAsyncPhysicsTickEnabled = bEnable;
}

void UActorComponent::DeferRemoveAsyncPhysicsTick()
{
	if (FPhysScene_Chaos* Scene = static_cast<FPhysScene_Chaos*>(GetPhysicsScene()))
	{
		// Set 0 for the step so that it gets run immediately on the next async tick.
		Scene->EnqueueAsyncPhysicsCommand(0, this,
			[this]()
			{
				SetAsyncPhysicsTickEnabled(false);
			}
		,false);
	}
}

void UActorComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	check(bRegistered);

	if (GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint) || !GetClass()->HasAnyClassFlags(CLASS_Native))
	{
		ReceiveTick(DeltaTime);

		if (GTickComponentLatentActionsWithTheComponent)
		{
			// Update any latent actions we have for this component, this will update even if paused if bUpdateWhilePaused is enabled
			// If this tick is skipped on a frame because we've got a TickInterval, our latent actions will be ticked
			// anyway by UWorld::Tick(). Given that, our latent actions don't need to be passed a larger
			// DeltaSeconds to make up the frames that they missed (because they wouldn't have missed any).
			// So pass in the world's DeltaSeconds value rather than our specific DeltaSeconds value.
			if (UWorld* ComponentWorld = GetWorld())
			{
				ComponentWorld->GetLatentActionManager().ProcessLatentActions(this, ComponentWorld->GetDeltaSeconds());
			}
		}
	}
}

void UActorComponent::PreRegisterComponentWithWorld(UWorld* InWorld)
{
	checkf(!IsUnreachable(), TEXT("%s"), *GetFullName());

	if (!IsValidChecked(this))
	{
		UE_LOGF(LogActorComponent, Log, "Trying to pre-register component with IsValid() == false. Aborting.");
		return;
	}

	if (IsPreRegistering())
	{
		UE_LOGF(LogActorComponent, Log, "Component already pre-registering. Aborting.");
		return;
	}

	// If the component was already pre-registered, do nothing
	if (IsPreRegistered())
	{
		UE_LOGF(LogActorComponent, Log, "Component already pre-registered. Aborting.");
		return;
	}

	if (InWorld == nullptr)
	{
		return;
	}

	// If not pre-registered, should not have a world
	checkf(WorldPrivate == nullptr, TEXT("%s"), *GetFullName());

	AActor* MyOwner = GetOwner();
#if DO_CHECK || USING_CODE_ANALYSIS
	checkSlow(MyOwner == nullptr || MyOwner->OwnsComponent(this));
#endif

	if (MyOwner && MyOwner->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
	{
		UE_LOGF(LogActorComponent, Log, "Component owner belongs to a DEADCLASS");
		return;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Can only pre-register with an Actor if we are created within one
	if (MyOwner)
	{
		checkf(!MyOwner->IsUnreachable(), TEXT("%s"), *GetFullName());
		// The only time you should specify a scene that is not Owner->GetWorld() is when you don't have an Actor
		UE_CLOGF(InWorld != MyOwner->GetWorld(), LogActorComponent, Log, "Specifying a world, but an Owner Actor found, and InWorld is not GetOwner()->GetWorld()");
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	if (!bHasBeenCreated)
	{
		OnComponentCreated();
	}

	WorldPrivate = InWorld;

	OnPreRegister();
}

void UActorComponent::RegisterComponentWithWorld(UWorld* InWorld, FRegisterComponentContext* Context)
{
	SCOPE_CYCLE_COUNTER(STAT_RegisterComponent);
	FScopeCycleCounterUObject ComponentScope(this);

	checkf(!IsUnreachable(), TEXT("%s"), *GetFullName());

	if(!IsValidChecked(this))
	{
		UE_LOGF(LogActorComponent, Log, "RegisterComponentWithWorld: (%ls) Trying to register component with IsValid() == false. Aborting.", *GetPathName());
		return;
	}

	// If the component was already registered, do nothing
	if(IsRegistered())
	{
		UE_LOGF(LogActorComponent, Log, "RegisterComponentWithWorld: (%ls) Already registered. Aborting.", *GetPathName());
		return;
	}

	if(InWorld == nullptr)
	{
		//UE_LOGF(LogActorComponent, Log, "RegisterComponentWithWorld: (%ls) NULL InWorld specified. Aborting.", *GetPathName());
		return;
	}

	if (!IsValid(InWorld) || InWorld->IsCleanedUp())
	{
		UE_LOGF(LogActorComponent, Error, "RegisterComponentWithWorld: (%ls) Trying to register component in invalid or cleaned up world. Aborting.", *GetPathName());
		return;
	}

	// If not registered/pre-registered, should not have a world
	checkf(WorldPrivate == nullptr || (IsPreRegistered()), TEXT("%s"), *GetFullName());

	AActor* MyOwner = GetOwner();
#if DO_CHECK || USING_CODE_ANALYSIS
	checkSlow(MyOwner == nullptr || MyOwner->OwnsComponent(this));
#endif

	if (MyOwner && MyOwner->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
	{
		UE_LOGF(LogActorComponent, Log, "RegisterComponentWithWorld: Owner belongs to a DEADCLASS %ls", *GetPathNameSafe(MyOwner->GetClass()));
		return;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Can only register with an Actor if we are created within one
	if(MyOwner)
	{
		checkf(!MyOwner->IsUnreachable(), TEXT("%s"), *GetFullName());
		if(InWorld != MyOwner->GetWorld())
		{
			// The only time you should specify a scene that is not Owner->GetWorld() is when you don't have an Actor
			UE_LOGF(LogActorComponent, Log, "RegisterComponentWithWorld: (%ls) Specifying a world, but an Owner Actor found, and InWorld is not GetOwner()->GetWorld()", *GetPathName());
		}
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	if (!bHasBeenCreated)
	{
		OnComponentCreated();
	}

	WorldPrivate = InWorld;

	ExecuteRegisterEvents(Context);

	// If not in a game world register ticks now, otherwise defer until BeginPlay. If no owner we won't trigger BeginPlay either so register now in that case as well.
	if (!InWorld->IsGameWorld())
	{
		RegisterAllComponentTickFunctions(true);
	}
	else if (MyOwner == nullptr)
	{
		if (!bHasBeenInitialized && bWantsInitializeComponent)
		{
			InitializeComponent();
		}

		RegisterAllComponentTickFunctions(true);
	}
	else
	{
		MyOwner->HandleRegisterComponentWithWorld(this);
	}

	// If this is a blueprint created component and it has component children they can miss getting registered in some scenarios
	if (IsCreatedByConstructionScript())
	{
		TArray<UObject*> Children;
		GetObjectsWithOuter(this, Children, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);

		for (UObject* Child : Children)
		{
			if (UActorComponent* ChildComponent = Cast<UActorComponent>(Child))
			{
				if (ChildComponent->bAutoRegister && !ChildComponent->IsRegistered() && ChildComponent->GetOwner() == MyOwner)
				{
					ChildComponent->RegisterComponentWithWorld(InWorld);
				}
			}
		}

	}
	
	if (MyOwner && MyOwner->InputComponent)
	{
		UInputDelegateBinding::BindInputDelegates(GetClass(), MyOwner->InputComponent, this);
	}
}

void UActorComponent::RegisterComponent()
{
	AActor* MyOwner = GetOwner();
	UWorld* MyOwnerWorld = (MyOwner ? MyOwner->GetWorld() : nullptr);
	if (ensure(MyOwnerWorld))
	{
		//@note FH: world should be initialized when calling RegisterComponent or it should be handled gracefully as a no-op but that isn't currently the case
		// however a lot of legacy code may end up calling RegisterComponent prior to world initialization hence why the ensure is currently commented
		//ensure(MyOwnerWorld->bIsWorldInitialized);
		RegisterComponentWithWorld(MyOwnerWorld);
	}
}

void UActorComponent::PreUnregisterComponentFromWorld()
{
	check(RegistrationState == EComponentRegistrationState::None);
	OnPreUnregister();
	check(IsPreUnregistering());
}

void UActorComponent::OnPreUnregister()
{
	checkf(WorldPrivate, TEXT("OnPreUnregister: %s to %s"), *GetDetailedInfo(), *GetFullNameSafe(GetOwner()));
	checkf(RegistrationState == EComponentRegistrationState::None, TEXT("OnPreUnregister: %s to %s"), *GetDetailedInfo(), *GetFullNameSafe(GetOwner()));
	checkf(bRegistered, TEXT("OnPreUnregister: %s to %s"), *GetDetailedInfo(), *GetFullNameSafe(GetOwner()));

	RegistrationState = EComponentRegistrationState::PreUnregistering;

	AActor* MyOwner = GetOwner();
	if (ULevel* Level = MyOwner ? MyOwner->GetLevel() : nullptr)
	{
		FLevelRegistrationAccessor::OnPreUnregisterComponent(Level, this);
	}

	check(ShouldAsyncDestroyPhysicsState());
	FPhysScene_Chaos* PhysScene = GetPhysicsScene();
	check(PhysScene);
	check(!IsAsyncCreatePhysicsStateRunning());
	check(!IsAsyncDestroyPhysicsStateRunning());
	PhysicsStateAsyncCreationState = EPhysicsStateAsyncCreationState::Destroying;
	PhysicsStateAsyncCreationScene = PhysScene;
	verify(PhysScene->PushAsyncDestroyPhysicsStateJob(this));
	check(IsAsyncDestroyPhysicsStateRunning());
}

void UActorComponent::UnregisterComponent()
{
	SCOPE_CYCLE_COUNTER(STAT_UnregisterComponent);
	FScopeCycleCounterUObject ComponentScope(this);

	// Do nothing if not registered
	if(!IsRegistered())
	{
		UE_LOGF(LogActorComponent, Log, "UnregisterComponent: (%ls) Not registered. Aborting.", *GetPathName());
		return;
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (UE::RemoteObject::Handle::GetResidence(this) == EResidence::LocalNotReady)
	{
		UE_LOGF(LogActorComponent, Log, "UnregisterComponent: (%ls) has LocalNotReady residence. Skipping unregistration.", *GetPathName());
		return;
	}
#endif

	// If registered, should have a world
	checkf(WorldPrivate != nullptr, TEXT("%s"), *GetFullName());

	RegisterAllComponentTickFunctions(false);
	ExecuteUnregisterEvents();

	WorldPrivate = nullptr;
}

void UActorComponent::DestroyComponent(bool bPromoteChildren/*= false*/)
{
	// Avoid re-entrancy
	if (bIsBeingDestroyed)
	{
		return;
	}

	bIsBeingDestroyed = true;

	if (bHasBegunPlay)
	{
		EndPlay(EEndPlayReason::Destroyed);
	}

	// Ensure that we call UninitializeComponent before we destroy this component
	if (bHasBeenInitialized)
	{
		UninitializeComponent();
	}

	bIsReadyForReplication = false;

	// Unregister if registered
	if(IsRegistered())
	{
		UnregisterComponent();
	}

	// Then remove from Components array, if we have an Actor
	if(AActor* MyOwner = GetOwner())
	{
		if (IsCreatedByConstructionScript())
		{
			MyOwner->BlueprintCreatedComponents.Remove(this);
		}
		else
		{
			MyOwner->RemoveInstanceComponent(this);
		}
		MyOwner->RemoveOwnedComponent(this);
		if (MyOwner->GetRootComponent() == this)
		{
			MyOwner->SetRootComponent(NULL);
		}
	}

	// Tell the component it is being destroyed
	OnComponentDestroyed(false);

	// Finally mark pending kill, to NULL out any other refs
	MarkAsGarbage();
}

void UActorComponent::OnComponentCreated()
{
	ensure(!bHasBeenCreated);
	bHasBeenCreated = true;
}

void UActorComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	// @TODO: Would be nice to ensure(bHasBeenCreated), but there are still many places where components are created without calling OnComponentCreated
	bHasBeenCreated = false;

#if WITH_EDITOR
	{
		using namespace UE::Editor::DataStorage;
		if (ICompatibilityProvider* Storage = UE::Editor::DataStorage::GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
		{
			Storage->RemoveCompatibleObject(this);
		}
	}
#endif
}

void UActorComponent::K2_DestroyComponent(UObject* Object)
{
	AActor* MyOwner = GetOwner();
	if (bAllowAnyoneToDestroyMe || Object == this || MyOwner == NULL || MyOwner == Object)
	{
		DestroyComponent();
	}
	else
	{
		// TODO: Put in Message Log
		UE_LOGF(LogActorComponent, Error, "May not destroy component %ls owned by %ls.", *GetFullName(), *MyOwner->GetFullName());
	}
}

void UActorComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	check(IsRegistered());
	check(WorldPrivate->Scene);
	check(!bRenderStateCreated);
	bRenderStateCreated = true;

	bRenderStateDirty = false;
	bRenderTransformDirty = false;
	bRenderDynamicDataDirty = false;
	bRenderInstancesDirty = false;

#if LOG_RENDER_STATE
	UE_LOGF(LogActorComponent, Log, "CreateRenderState_Concurrent: %ls", *GetPathName());
#endif

#if WITH_EDITOR
	FObjectCacheEventSink::NotifyRenderStateChanged_Concurrent(this);
#endif
}

void UActorComponent::SendRenderTransform_Concurrent()
{
	check(bRenderStateCreated);
	bRenderTransformDirty = false;

#if LOG_RENDER_STATE
	UE_LOGF(LogActorComponent, Log, "SendRenderTransform_Concurrent: %ls", *GetPathName());
#endif
}

void UActorComponent::SendRenderDynamicData_Concurrent()
{
	check(bRenderStateCreated);
	bRenderDynamicDataDirty = false;

#if LOG_RENDER_STATE
	UE_LOGF(LogActorComponent, Log, "SendRenderDynamicData_Concurrent: %ls", *GetPathName());
#endif
}

void UActorComponent::SendRenderInstanceData_Concurrent()
{
	check(bRenderStateCreated);
	bRenderInstancesDirty = false;

#if LOG_RENDER_STATE
	UE_LOGF(LogActorComponent, Log, "SendRenderInstanceData_Concurrent: %ls", *GetPathName());
#endif
}

void UActorComponent::DestroyRenderState_Concurrent()
{
	check(bRenderStateCreated);
	bRenderStateCreated = false;

	// Also reset other dirty states
	// There is a path in the engine that immediately unregisters the component after registration (AActor::RerunConstructionScripts())
	// so that the component can be left in a state where its transform is marked for update while render state destroyed
	bRenderStateDirty = false;
	bRenderTransformDirty = false;
	bRenderInstancesDirty = false;
	bRenderDynamicDataDirty = false;

#if LOG_RENDER_STATE
	UE_LOGF(LogActorComponent, Log, "DestroyRenderState_Concurrent: %ls", *GetPathName());
#endif

#if WITH_EDITOR
	FObjectCacheEventSink::NotifyRenderStateChanged_Concurrent(this);
#endif
}

void UActorComponent::OnCreatePhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorComponent::OnCreatePhysicsState);
	check(IsRegistered() || IsPreRegistering());
	check(ShouldCreatePhysicsState());
	check(GetPhysicsScene());
	check(!bPhysicsStateCreated);
	bPhysicsStateCreated = true;
}

bool UActorComponent::IsAsyncPhysicsStateCreated() const
{
	return IsPhysicsStateCreated();
}

UObject* UActorComponent::GetAsyncPhysicsStateObject() const
{
	return const_cast<UActorComponent*>(this);
}

bool UActorComponent::OnAsyncCreatePhysicsState(const UE::FTimeout& Timeout)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorComponent::OnAsyncCreatePhysicsState);
	check(AllowsAsyncPhysicsStateCreation());
	OnCreatePhysicsState();
	return true;
}

void UActorComponent::OnAsyncCreatePhysicsStateEnd_GameThread()
{
	check(IsPhysicsStateCreated());
	check(IsPreRegistering())
	check(IsAsyncCreatePhysicsStateRunning());
	check(!IsA<UPrimitiveComponent>() || !Cast<UPrimitiveComponent>(this)->DeferredCreatePhysicsStateScene);

	// Reset members used by asynchronous physics state creation
	PhysicsStateAsyncCreationState = EPhysicsStateAsyncCreationState::None;
	PhysicsStateAsyncCreationScene = nullptr;
	bCallAsyncPhysicsStateCreatedDelegate = true;

	OnPreRegistered();
	check(IsPreRegistered());
}

void UActorComponent::OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects)
{
	GlobalDestroyPhysicsDelegate.Broadcast(this);
	ensureMsgf(bRegistered, TEXT("Component has physics state when not registered (%s)"), *GetFullName()); // should not have physics state unless we are registered
}

void UActorComponent::OnAsyncDestroyPhysicsStateEnd_GameThread()
{
	check(IsAsyncDestroyPhysicsStateRunning());
	PhysicsStateAsyncCreationState = EPhysicsStateAsyncCreationState::None;
	PhysicsStateAsyncCreationScene = nullptr;

	checkf(!HasValidPhysicsState(), TEXT("Failed to destroy physics state (%s)"), *GetFullName());
	check(bPhysicsStateCreated);
	bPhysicsStateCreated = false;

	OnPreUnregistered();
	check(IsPreUnregistered());
}

void UActorComponent::OnPreUnregistered()
{
	check(IsPreUnregistering());
	RegistrationState = EComponentRegistrationState::PreUnregistered;

	AActor* MyOwner = GetOwner();
	if (ULevel* Level = MyOwner ? MyOwner->GetLevel() : nullptr)
	{
		FLevelRegistrationAccessor::OnPreUnregisteredComponent(Level, this);
	}
}

void UActorComponent::OnDestroyPhysicsState()
{
	ensure(bPhysicsStateCreated);
	bPhysicsStateCreated = false;
}

void UActorComponent::CreatePhysicsState(bool bAllowDeferral)
{
	LLM_SCOPE(ELLMTag::Chaos);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(UWorld::StaticClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, UWorld::StaticClass()->GetFName(), GetPackage()->GetFName());

	SCOPE_CYCLE_COUNTER(STAT_ComponentCreatePhysicsState);
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorComponent::CreatePhysicsState);

	if (!bPhysicsStateCreated && GetPhysicsScene() && ShouldCreatePhysicsState())
	{
		UPrimitiveComponent* Primitive = Cast<UPrimitiveComponent>(this);

		bool ShouldDefer = false;
		if (UWorld* World = GetWorld())
		{
			if (World->GetAllowDeferredPhysicsStateCreation())
			{
				if (GEnableDeferredPhysicsCreation && bAllowDeferral && Primitive && !Primitive->GetGenerateOverlapEvents())
				{
					if (UBodySetup* Setup = Primitive->GetBodySetup())
					{
						if (!Setup->bCreatedPhysicsMeshes)
						{
							ShouldDefer = true;
						}
					}
				}
			}

		}

		if (ShouldDefer)
		{
			GetPhysicsScene()->DeferPhysicsStateCreation(Primitive);
		}
		else
		{
			// Call virtual
			OnCreatePhysicsState();

			checkf(bPhysicsStateCreated, TEXT("Failed to route OnCreatePhysicsState (%s)"), *GetFullName());

			// Broadcast delegate
			GlobalCreatePhysicsDelegate.Broadcast(this);
		}
	}
	// Asynchronous physics state was created, but GlobalCreatePhysicsDelegate needs to be broadcasted
	else if (bCallAsyncPhysicsStateCreatedDelegate)
	{
		check(bPhysicsStateCreated);
		bCallAsyncPhysicsStateCreatedDelegate = false;
		if (ensure(GetPhysicsScene()))
		{
			GlobalCreatePhysicsDelegate.Broadcast(this);
		}
	}
}

void UActorComponent::DestroyPhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorComponent::DestroyPhysicsState);
	SCOPE_CYCLE_COUNTER(STAT_ComponentDestroyPhysicsState);

	if (IsAsyncCreatePhysicsStateRunning())
	{
		check(!IsAsyncDestroyPhysicsStateRunning());
		check(!IsA<UPrimitiveComponent>() || !Cast<UPrimitiveComponent>(this)->DeferredCreatePhysicsStateScene);

		// RemoveAsyncPhysicsStateJobs invokes OnPostExecute_GameThread on the cancelled Create job,
		// which routes to OnAsyncCreatePhysicsStateEnd_GameThread and performs the full state cleanup:
		// it clears the PhysicsStateAsyncCreation* members, transitions the component to PreRegistered
		// via OnPreRegistered (which removes it from ULevel::PreRegisteringActorComponents), and sets
		// bCallAsyncPhysicsStateCreatedDelegate to request a deferred Create delegate broadcast.
		PhysicsStateAsyncCreationScene->RemoveAsyncPhysicsStateJobs(this);

		// Suppress the deferred Create broadcast: the bPhysicsStateCreated branch below is about
		// to tear the body down, so listeners would just see a paired Create/Destroy with no
		// observable lifetime in between. Clearing the flag also prevents a later sync
		// CreatePhysicsState from firing a spurious broadcast via its bCallAsyncPhysicsStateCreatedDelegate branch.
		bCallAsyncPhysicsStateCreatedDelegate = false;
		// Reset other members used by asynchronous physics state creation to be sure we are in the right state
		PhysicsStateAsyncCreationState = EPhysicsStateAsyncCreationState::None;
		PhysicsStateAsyncCreationScene = nullptr;
	}

	if (bPhysicsStateCreated)
	{
		check(!IsAsyncDestroyPhysicsStateRunning());
		// Broadcast delegate
		GlobalDestroyPhysicsDelegate.Broadcast(this);

		ensureMsgf(bRegistered || IsPreRegistered(), TEXT("Component has physics state when not registered (%s)"), *GetFullName()); // should not have physics state unless we are registered

		// Call virtual
		OnDestroyPhysicsState();

		checkf(!bPhysicsStateCreated, TEXT("Failed to route OnDestroyPhysicsState (%s)"), *GetFullName());
		checkf(!HasValidPhysicsState(), TEXT("Failed to destroy physics state (%s)"), *GetFullName());
	}
	else if(GEnableDeferredPhysicsCreation)
	{
		UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(this);
		if (PrimitiveComponent && PrimitiveComponent->DeferredCreatePhysicsStateScene != nullptr)
		{
			// We had to cache this scene because World ptr is null as we have unregistered already.
			PrimitiveComponent->DeferredCreatePhysicsStateScene->RemoveDeferredPhysicsStateCreation(PrimitiveComponent);
		}
	}
}

void UActorComponent::ExecuteRegisterEvents(FRegisterComponentContext* Context)
{
	if(!bRegistered)
	{
		SCOPE_CYCLE_COUNTER(STAT_ComponentOnRegister);
		OnRegister();
		checkf(bRegistered, TEXT("Failed to route OnRegister (%s)"), *GetFullName());
	}

	if(FApp::CanEverRender() && !bRenderStateCreated && WorldPrivate->Scene && ShouldCreateRenderState())
	{
		SCOPE_CYCLE_COUNTER(STAT_ComponentCreateRenderState);
		LLM_SCOPE(ELLMTag::SceneRender);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(UWorld::StaticClass(), ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, UWorld::StaticClass()->GetFName(), GetPackage()->GetFName());
		CreateRenderState_Concurrent(Context);
		checkf(bRenderStateCreated, TEXT("Failed to route CreateRenderState_Concurrent (%s)"), *GetFullName());
	}

	CreatePhysicsState(/*bAllowDeferral=*/true);
}


void UActorComponent::ExecuteUnregisterEvents()
{
	DestroyPhysicsState();

	if (bRenderStateCreated)
	{
		SCOPE_CYCLE_COUNTER(STAT_ComponentDestroyRenderState);
		checkf(bRegistered, TEXT("Component has render state when not registered (%s)"), *GetFullName());
		DestroyRenderState_Concurrent();
		checkf(!bRenderStateCreated, TEXT("Failed to route DestroyRenderState_Concurrent (%s)"), *GetFullName());
	}

	if (bRegistered)
	{
		SCOPE_CYCLE_COUNTER(STAT_ComponentOnUnregister);
		OnUnregister();
		checkf(!bRegistered, TEXT("Failed to route OnUnregister (%s)"), *GetFullName());
	}
}

void UActorComponent::ReregisterComponent()
{
	if (AllowReregistration())
	{
		if (!IsRegistered())
		{
			UE_LOGF(LogActorComponent, Log, "ReregisterComponent: (%ls) Not currently registered. Aborting.", *GetPathName());
			return;
		}

		FComponentReregisterContext(this);
	}
}

void UActorComponent::RecreateRenderState_Concurrent()
{
	bool bCanRecreate = IsRegistered() && WorldPrivate->Scene;

	if(bRenderStateCreated)
	{
		// Only set bRenderStateRecreating if we know for sure we are going to actually re-create it, so components can always count on the
		// calls happening in sequence if bRenderStateRecreating is set, and don't need to handle edge cases where the latter isn't called.
		if (bCanRecreate)
		{
			check(bRenderStateRecreating == false);
			bRenderStateRecreating = true;
		}
		check(IsRegistered()); // Should never have render state unless registered
		DestroyRenderState_Concurrent();
		checkf(!bRenderStateCreated, TEXT("Failed to route DestroyRenderState_Concurrent (%s)"), *GetFullName());
	}

	if (bCanRecreate)
	{
		CreateRenderState_Concurrent(nullptr);
		bRenderStateRecreating = false;
		checkf(bRenderStateCreated, TEXT("Failed to route CreateRenderState_Concurrent (%s)"), *GetFullName());
	}
}

void UActorComponent::RecreatePhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorComponent::RecreatePhysicsState);
	check(!IsAsyncCreatePhysicsStateRunning());
	
	DestroyPhysicsState();

	if (IsRegistered())
	{
		CreatePhysicsState();
	}
}

void UActorComponent::SetTickGroup(ETickingGroup NewTickGroup)
{
	PrimaryComponentTick.TickGroup = NewTickGroup;
}


void UActorComponent::AddTickPrerequisiteActor(AActor* PrerequisiteActor)
{
	if (PrimaryComponentTick.bCanEverTick && PrerequisiteActor && PrerequisiteActor->PrimaryActorTick.bCanEverTick)
	{
		PrimaryComponentTick.AddPrerequisite(PrerequisiteActor, PrerequisiteActor->PrimaryActorTick);
	}
}

void UActorComponent::AddTickPrerequisiteComponent(UActorComponent* PrerequisiteComponent)
{
	if (PrimaryComponentTick.bCanEverTick && PrerequisiteComponent && PrerequisiteComponent->PrimaryComponentTick.bCanEverTick)
	{
		PrimaryComponentTick.AddPrerequisite(PrerequisiteComponent, PrerequisiteComponent->PrimaryComponentTick);
	}
}

void UActorComponent::RemoveTickPrerequisiteActor(AActor* PrerequisiteActor)
{
	if (PrerequisiteActor)
	{
		PrimaryComponentTick.RemovePrerequisite(PrerequisiteActor, PrerequisiteActor->PrimaryActorTick);
	}
}

void UActorComponent::RemoveTickPrerequisiteComponent(UActorComponent* PrerequisiteComponent)
{
	if (PrerequisiteComponent)
	{
		PrimaryComponentTick.RemovePrerequisite(PrerequisiteComponent, PrerequisiteComponent->PrimaryComponentTick);
	}
}

void UActorComponent::DoDeferredRenderUpdates_Concurrent()
{
	LLM_SCOPE(ELLMTag::SceneRender);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(UWorld::StaticClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, UWorld::StaticClass()->GetFName(), GetPackage()->GetFName());

	checkf(!IsUnreachable(), TEXT("%s"), *GetFullName());
	checkf(!IsTemplate(), TEXT("%s"), *GetFullName());
	checkf(IsValidChecked(this), TEXT("%s"), *GetFullName());

	FScopeCycleCounterUObject ContextScope(this);
	FScopeCycleCounterUObject AdditionalScope(STATS ? AdditionalStatObject() : nullptr);

	if(!IsRegistered())
	{
		UE_LOGF(LogActorComponent, Log, "UpdateComponent: (%ls) Not registered, Aborting.", *GetPathName());
		return;
	}

	if(bRenderStateDirty)
	{
		SCOPE_CYCLE_COUNTER(STAT_PostTickComponentRecreate);
		RecreateRenderState_Concurrent();
		checkf(!bRenderStateDirty, TEXT("Failed to route CreateRenderState_Concurrent (%s)"), *GetFullName());
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_PostTickComponentLW);
		if(bRenderTransformDirty)
		{
			// Update the component's transform if the actor has been moved since it was last updated.
			SendRenderTransform_Concurrent();
		}

		if(bRenderDynamicDataDirty)
		{
			SendRenderDynamicData_Concurrent();
		}

		if (bRenderInstancesDirty)
		{
			SendRenderInstanceData_Concurrent();
		}
	}
}


void UActorComponent::MarkRenderStateDirty()
{
	// If registered and has a render state to mark as dirty
	if(IsRegistered() && bRenderStateCreated && (!bRenderStateDirty || !GetWorld()))
	{
		// Flag as dirty
		bRenderStateDirty = true;
		MarkForNeededEndOfFrameRecreate();

		MarkRenderStateDirtyEvent.Broadcast(*this);

#if WITH_EDITOR
		FObjectCacheEventSink::NotifyRenderStateChanged_Concurrent(this);
#endif
	}
}


void UActorComponent::MarkRenderTransformDirty()
{
	if (IsRegistered() && bRenderStateCreated)
	{
		bRenderTransformDirty = true;
		MarkForNeededEndOfFrameUpdate();
	}
}

void UActorComponent::MarkRenderInstancesDirty()
{
	if (IsRegistered() && bRenderStateCreated)
	{
		bRenderInstancesDirty = true;
		MarkForNeededEndOfFrameUpdate();
	}
}


void UActorComponent::MarkRenderDynamicDataDirty()
{
	// If registered and has a render state to mark as dirty
	if(IsRegistered() && bRenderStateCreated)
	{
		// Flag as dirty
		bRenderDynamicDataDirty = true;
		MarkForNeededEndOfFrameUpdate();
	}
}

void UActorComponent::MarkForNeededEndOfFrameUpdate(bool bReadyForEarlyUpdate)
{
	if (bNeverNeedsRenderUpdate)
	{
		return;
	}

	UWorld* ComponentWorld = GetWorld();
	if (ComponentWorld)
	{
		if (bReadyForEarlyUpdate)
		{
			if (!bReadyForEarlyEndOfFrameUpdate && MarkedForEndOfFrameUpdateArrayIndex != INDEX_NONE)
			{
				ComponentWorld->ClearActorComponentEndOfFrameUpdate(this);
			}

			bReadyForEarlyEndOfFrameUpdate = true;
		}

		ComponentWorld->MarkActorComponentForNeededEndOfFrameUpdate(this, RequiresGameThreadEndOfFrameUpdates());
	}
	else if (!IsUnreachable())
	{
		// we don't have a world, do it right now.
		DoDeferredRenderUpdates_Concurrent();
	}
}

void UActorComponent::ClearNeedEndOfFrameUpdate_Internal()
{
	if (UWorld* World = GetWorld())
	{
		World->ClearActorComponentEndOfFrameUpdate(this);
	}
}

void UActorComponent::MarkForNeededEndOfFrameRecreate()
{
	if (bNeverNeedsRenderUpdate)
	{
		return;
	}

	UWorld* ComponentWorld = GetWorld();
	if (ComponentWorld)
	{
		// by convention, recreates are always done on the gamethread
		ComponentWorld->MarkActorComponentForNeededEndOfFrameUpdate(this, RequiresGameThreadEndOfFrameRecreate());
	}
	else if (!IsUnreachable())
	{
		// we don't have a world, do it right now.
		DoDeferredRenderUpdates_Concurrent();
	}
}

bool UActorComponent::RequiresGameThreadEndOfFrameUpdates() const
{
	return false;
}

bool UActorComponent::RequiresGameThreadEndOfFrameRecreate() const
{
	return true;
}

bool UActorComponent::RequiresPreEndOfFrameSync() const
{
	return false;
}

void UActorComponent::Activate(bool bReset)
{
	if (bReset || ShouldActivate()==true)
	{
		SetComponentTickEnabled(true);
		SetActiveFlag(true);

		OnComponentActivated.Broadcast(this, bReset);
	}
}

void UActorComponent::Deactivate()
{
	if (ShouldActivate()==false)
	{
		SetComponentTickEnabled(false);
		SetActiveFlag(false);

		OnComponentDeactivated.Broadcast(this);
	}
}

bool UActorComponent::ShouldActivate() const
{
	// if not active, should activate
	return !IsActive();
}

void UActorComponent::SetActive(bool bNewActive, bool bReset)
{
	// if it wants to activate
	if (bNewActive)
	{
		// make sure to check if it should activate
		Activate(bReset);	
	}
	// otherwise, make sure it shouldn't activate
	else 
	{
		Deactivate();
	}
}

void UActorComponent::SetAutoActivate(bool bNewAutoActivate)
{
	if (!bRegistered || IsOwnerRunningUserConstructionScript())
	{
		bAutoActivate = bNewAutoActivate;
	}
	else
	{
		UE_LOGF(LogActorComponent, Warning, "SetAutoActivate called on component %ls after construction!", *GetFullName());
	}
}

void UActorComponent::ToggleActive()
{
	SetActive(!IsActive());
}

void UActorComponent::SetTickableWhenPaused(bool bTickableWhenPaused)
{
	PrimaryComponentTick.bTickEvenWhenPaused = bTickableWhenPaused;
}

bool UActorComponent::IsOwnerRunningUserConstructionScript() const
{
	AActor* MyOwner = GetOwner();
	return (MyOwner && MyOwner->IsRunningUserConstructionScript());
}

void UActorComponent::AddAssetUserData( UAssetUserData* InUserData)
{
	if (InUserData != NULL)
	{
		RemoveUserDataOfClass(InUserData->GetClass());
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UActorComponent::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	const TArray<UAssetUserData*>* ArrayPtr = GetAssetUserDataArray();
	for (int32 DataIdx = 0; DataIdx < ArrayPtr->Num(); DataIdx++)
	{
		UAssetUserData* Datum = (*ArrayPtr)[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

const TArray<UAssetUserData*>* UActorComponent::GetAssetUserDataArray() const
{
#if WITH_EDITOR
	if (IsRunningCookCommandlet())
	{
		return &ToRawPtrTArrayUnsafe(AssetUserData);
	}
	else
	{
		static thread_local TArray<TObjectPtr<UAssetUserData>> CachedAssetUserData;
		CachedAssetUserData.Reset();
		CachedAssetUserData.Append(AssetUserData);
		CachedAssetUserData.Append(AssetUserDataEditorOnly);
		return &ToRawPtrTArrayUnsafe(CachedAssetUserData);
	}
#else
	return &ToRawPtrTArrayUnsafe(AssetUserData);
#endif
}

void UActorComponent::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
#if WITH_EDITOR
	for (int32 DataIdx = 0; DataIdx < AssetUserDataEditorOnly.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserDataEditorOnly[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserDataEditorOnly.RemoveAt(DataIdx);
			return;
		}
	}
#endif
}

void UActorComponent::OnCreatedFromReplication()
{
	RegisterComponent();
	SetIsReplicated(true);
}

void UActorComponent::OnDestroyedFromReplication()
{
	DestroyComponent();
}

void UActorComponent::SetNetAddressable()
{
	bNetAddressable = true;
}

bool UActorComponent::IsNameStableForNetworking() const
{
	/** 
	 * IsNameStableForNetworking means a component can be referred to its path name (relative to owning AActor*) over the network
	 *
	 * Components are net addressable if:
	 *	-They are Default Subobjects (created in C++ constructor)
	 *	-They were loaded directly from a package (placed in map actors)
	 *	-They were explicitly set to bNetAddressable (blueprint components created by SCS or UCS executed in the ConstructionScript only)
	 */

	return bNetAddressable || (Super::IsNameStableForNetworking() && (CreationMethod != EComponentCreationMethod::UserConstructionScript));
}

bool UActorComponent::IsSupportedForNetworking() const
{
	return GetIsReplicated() || IsNameStableForNetworking();
}

void UActorComponent::SetIsReplicated(bool bShouldReplicate)
{
	if (GetIsReplicated() != bShouldReplicate)
	{
		ensureMsgf(!NeedsInitialization(), TEXT("SetIsReplicatedByDefault is preferred during Component Construction."));

		if (GetComponentClassCanReplicate())
		{
			bReplicates = bShouldReplicate;
			MARK_PROPERTY_DIRTY_FROM_NAME(UActorComponent, bReplicates, this);

			if (AActor* MyOwner = GetOwner())
			{
				MyOwner->UpdateReplicatedComponent( this );
			}
		}
		else
		{
			UE_LOGF(LogActorComponent, Error, "Calling SetIsReplicated on component of Class '%ls' which cannot replicate.", *GetClass()->GetName());
		}
	}
}

void UActorComponent::AddReplicatedSubObject(UObject* SubObject, ELifetimeCondition NetCondition)
{
	if (AActor* MyOwner=GetOwner())
	{
		MyOwner->AddActorComponentReplicatedSubObject(this, SubObject, NetCondition);
	}
}

void UActorComponent::RemoveReplicatedSubObject(UObject* SubObject)
{
	if (AActor* MyOwner=GetOwner())
	{
		MyOwner->RemoveActorComponentReplicatedSubObject(this, SubObject);
	}
}

void UActorComponent::DestroyReplicatedSubObjectOnRemotePeers(UObject* SubObject)
{
	if (AActor* MyOwner = GetOwner())
	{
		MyOwner->DestroyReplicatedSubObjectOnRemotePeers(this, SubObject);
	}
}

void UActorComponent::TearOffReplicatedSubObjectOnRemotePeers(UObject* SubObject)
{
	if (AActor* MyOwner=GetOwner())
	{
		MyOwner->TearOffReplicatedSubObjectOnRemotePeers(this, SubObject);
	}
}

bool UActorComponent::IsReplicatedSubObjectRegistered(const UObject* SubObject) const
{
	if (AActor* MyOwner=GetOwner())
	{
		return MyOwner->IsActorComponentReplicatedSubObjectRegistered(this, SubObject);
	}

	return false;
}

bool UActorComponent::ReplicateSubobjects(UActorChannel* Channel, FOutBunch* Bunch, FReplicationFlags* RepFlags)
{
	UActorChannel::SetCurrentSubObjectOwner(this);
	return false;
}

bool UActorComponent::GetComponentClassCanReplicate() const
{
	return true;
}

ENetRole UActorComponent::GetOwnerRole() const
{
	AActor* MyOwner = GetOwner();
	return (MyOwner ? MyOwner->GetLocalRole() : ROLE_None);
}

void UActorComponent::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass());
	if (BPClass != NULL)
	{
		BPClass->GetLifetimeBlueprintReplicationList(OutLifetimeProps);
	}

	FDoRepLifetimeParams SharedParams;
	SharedParams.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS_FAST(UActorComponent, bIsActive, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(UActorComponent, bReplicates, SharedParams);
}

void UActorComponent::OnRep_IsActive()
{
	SetComponentTickEnabled(IsActive());
}

#if WITH_EDITOR
bool UActorComponent::CanEditChange(const FProperty* InProperty) const
{
	if (AActor* Owner = GetOwner())
	{
		if (!Owner->CanEditChangeComponent(this, InProperty))
		{
			return false;
		}
	}
	if (Super::CanEditChange(InProperty))
	{
		UActorComponent* ComponentArchetype = Cast<UActorComponent>(GetArchetype());
		if (ComponentArchetype == nullptr || ComponentArchetype->bEditableWhenInherited)
		{
			return true;
		}
	}
	return false;
}
#endif

void UActorComponent::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	if (CreationMethod == EComponentCreationMethod::UserConstructionScript)
	{
		if (!IsNameStableForNetworking() && GetArchetype() != GetClass()->GetDefaultObject())
		{
			RegistrationFlags |= UE::Net::EFragmentRegistrationFlags::InitializeDefaultStateFromClassDefaults;
			UE_LOGF(LogIris, Warning, "The default state of replicated dynamic component %ls::%ls will be built using the class CDO instead of the archetype. The non-replicated properties of the component on clients may be initialized wrong.", *GetNameSafe(GetOwner()), *GetName());
		}
	}
	
	// Build descriptors and allocate PropertyReplicationFragments for this object
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UActorComponent::OnReplicationStartedForIris(const FOnReplicationStartedParams&)
{
}

void UActorComponent::OnStopReplicationForIris(const FOnStopReplicationParams&)
{
}

bool UActorComponent::IsEditableWhenInherited() const
{
	bool bCanEdit = bEditableWhenInherited;
	if (bCanEdit)
	{
#if WITH_EDITOR
		if (CreationMethod == EComponentCreationMethod::Native && !IsTemplate())
		{
			bCanEdit = FComponentEditorUtils::GetPropertyForEditableNativeComponent(this) != nullptr;
		}
		else
#endif
		if (CreationMethod == EComponentCreationMethod::UserConstructionScript)
		{
			bCanEdit = false;
		}
	}
	return bCanEdit;
}

void UActorComponent::DetermineUCSModifiedProperties()
{
	if (CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
	{
		TArray<FSimpleMemberReference> UCSModifiedProperties;

		class FComponentPropertySkipper : public FArchive
		{
		public:
			FComponentPropertySkipper()
				: FArchive()
			{
				this->SetIsSaving(true);

				// Include properties that would normally skip tagged serialization (e.g. bulk serialization of array properties).
				ArPortFlags |= PPF_ForceTaggedSerialization;
			}

			virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
			{
				static const FName MD_SkipUCSModifiedProperties("SkipUCSModifiedProperties");
				return (InProperty->HasAnyPropertyFlags(CPF_Transient)
					|| !InProperty->HasAnyPropertyFlags(CPF_Edit | CPF_Interp)
					|| InProperty->IsA<FMulticastDelegateProperty>()
#if WITH_EDITOR
					|| InProperty->HasMetaData(MD_SkipUCSModifiedProperties)
#endif
					);
			}
		} PropertySkipper;

		UClass* ComponentClass = GetClass();
		UObject* ComponentArchetype = GetArchetype();

		for (TFieldIterator<FProperty> It(ComponentClass); It; ++It)
		{
			FProperty* Property = *It;
			if( Property->ShouldSerializeValue(PropertySkipper) )
			{
				for( int32 Idx=0; Idx<Property->ArrayDim; Idx++ )
				{
					uint8* DataPtr      = Property->ContainerPtrToValuePtr           <uint8>((uint8*)this, Idx);
					uint8* DefaultValue = Property->ContainerPtrToValuePtrForDefaults<uint8>(ComponentClass, (uint8*)ComponentArchetype, Idx);
					if (!Property->Identical( DataPtr, DefaultValue, PPF_DeepCompareInstances))
					{
						UCSModifiedProperties.Add(FSimpleMemberReference());
						FMemberReference::FillSimpleMemberReference<FProperty>(Property, UCSModifiedProperties.Last());
						break;
					}
				}
			}
		}

		UE::TRWScopeLock Lock(AllUCSModifiedPropertiesLock, SLT_Write);
		if (UCSModifiedProperties.Num() > 0)
		{
			AllUCSModifiedProperties.Add(this, MoveTemp(UCSModifiedProperties));
		}
		else
		{
			AllUCSModifiedProperties.Remove(this);
		}
	}
}

void UActorComponent::GetUCSModifiedProperties(TSet<const FProperty*>& ModifiedProperties) const
{
	UE::TRWScopeLock Lock(AllUCSModifiedPropertiesLock, SLT_ReadOnly);
	if (TArray<FSimpleMemberReference>* UCSModifiedProperties = AllUCSModifiedProperties.Find(this))
	{
		for (const FSimpleMemberReference& MemberReference : *UCSModifiedProperties)
		{
			ModifiedProperties.Add(FMemberReference::ResolveSimpleMemberReference<FProperty>(MemberReference));
		}
	}
}

void UActorComponent::RemoveUCSModifiedProperties(const TArray<FProperty*>& Properties)
{
	UE::TRWScopeLock Lock(AllUCSModifiedPropertiesLock, SLT_Write);
	if (TArray<FSimpleMemberReference>* UCSModifiedProperties = AllUCSModifiedProperties.Find(this))
	{
		for (FProperty* Property : Properties)
		{
			FSimpleMemberReference MemberReference;
			FMemberReference::FillSimpleMemberReference<FProperty>(Property, MemberReference);
			UCSModifiedProperties->RemoveSwap(MemberReference);
		}
	}
}

void UActorComponent::ClearUCSModifiedProperties()
{
	UE::TRWScopeLock Lock(AllUCSModifiedPropertiesLock, SLT_Write);
	AllUCSModifiedProperties.Remove(this);
}

void UActorComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UE::TRWScopeLock Lock(AllUCSModifiedPropertiesLock, SLT_ReadOnly);
	if (TArray<FSimpleMemberReference>* UCSModifiedProperties = AllUCSModifiedProperties.Find(CastChecked<UActorComponent>(InThis)))
	{
		for (FSimpleMemberReference& MemberReference : *UCSModifiedProperties)
		{
			Collector.AddReferencedObject(MemberReference.MemberParent);
		}
	}
}

void UActorComponent::SetCanEverAffectNavigation(bool bRelevant)
{
	if (bCanEverAffectNavigation != bRelevant)
	{
		bCanEverAffectNavigation = bRelevant;

		HandleCanEverAffectNavigationChange();
	}
}

void UActorComponent::HandleCanEverAffectNavigationChange(bool bForceUpdate)
{
	// update octree if already registered
	if (bRegistered || bForceUpdate)
	{
		if (bCanEverAffectNavigation)
		{
			// Update cached value
			bNavigationRelevant = IsNavigationRelevant();

			// Notify the navigation system
			FNavigationSystem::OnComponentRegistered(*this);
		}
		else
		{
			// Update cached value
			bNavigationRelevant = false;

			// Notify the navigation system
			FNavigationSystem::OnComponentUnregistered(*this);
		}
	}
}

void UActorComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

	if (Ar.IsLoading() && (Ar.HasAnyPortFlags(PPF_DuplicateForPIE)||!Ar.HasAnyPortFlags(PPF_Duplicate)) && !IsTemplate())
	{
		bHasBeenCreated = true;
	}

	if (Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) >= FFortniteReleaseBranchCustomObjectVersion::ActorComponentUCSModifiedPropertiesSparseStorage)
	{
		if (Ar.IsLoading())
		{
			TArray<FSimpleMemberReference> UCSModifiedProperties;
			Ar << UCSModifiedProperties;

			UE::TRWScopeLock Lock(AllUCSModifiedPropertiesLock, SLT_Write);
			if (UCSModifiedProperties.Num() > 0)
			{
				AllUCSModifiedProperties.Add(this, MoveTemp(UCSModifiedProperties));
			}
			else
			{
				AllUCSModifiedProperties.Remove(this);
			}
		}
		else
		{
			UE::TRWScopeLock Lock(AllUCSModifiedPropertiesLock, SLT_ReadOnly);
			if (TArray<FSimpleMemberReference>* UCSModifiedProperties = AllUCSModifiedProperties.Find(this))
			{
				Ar << *UCSModifiedProperties;
			}
			else
			{
				TArray<FSimpleMemberReference> EmptyUCSModifiedProperties;
				Ar << EmptyUCSModifiedProperties;
			}
		}
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Ar.IsMigratingRemoteObjects())
	{
		uint8 ComponentFlags = 0;

		if (Ar.IsSaving())
		{
			ComponentFlags |= (bRegistered & 1)					<< 0;
			ComponentFlags |= (bHasBeenInitialized & 1)			<< 1;
			ComponentFlags |= (bTickFunctionsRegistered & 1)	<< 2;
			ComponentFlags |= (bHasBegunPlay & 1)				<< 3;
			ComponentFlags |= (bPhysicsStateCreated & 1) 		<< 4;
			ComponentFlags |= (IsComponentTickEnabled() & 1)	<< 5;
			// bIsReadyForReplication is intentionally not serialized here because it's reset in UActorComponent::PostMigrate when receiving.
		}

		Ar << ComponentFlags;

		if (Ar.IsLoading())
		{
			bRegistered 				= (ComponentFlags >> 0) & 1;
			bHasBeenInitialized 		= (ComponentFlags >> 1) & 1;
			bTickFunctionsRegistered 	= (ComponentFlags >> 2) & 1;
			bHasBegunPlay				= (ComponentFlags >> 3) & 1;
			bPhysicsStateCreated 		= (ComponentFlags >> 4) & 1;
			bComponentTickEnabled 		= (ComponentFlags >> 5) & 1;
		}
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
}

void UActorComponent::PostApplyToComponent()
{
	if (IsRegistered())
	{
		ReregisterComponent();
	}
}

AActor* UActorComponent::GetActorOwnerNoninline() const
{
	// This is defined out-of-line because AActor isn't defined where the inlined function is.

	return GetTypedOuter<AActor>();
}

void UActorComponent::SetIsReplicatedByDefault(const bool bNewReplicates)
{
	// Don't bother checking parent here.
	if (LIKELY(NeedsInitialization()))
	{
		bReplicates = bNewReplicates;
		MARK_PROPERTY_DIRTY_FROM_NAME(UActorComponent, bReplicates, this);
	}
	else
	{
		ensureMsgf(false, TEXT("SetIsReplicatedByDefault should only be called during Component Construction. Class=%s"), *GetPathNameSafe(GetClass()));
		SetIsReplicated(bNewReplicates);
	}
}

void UActorComponent::SetActiveFlag(const bool bNewIsActive)
{
	bIsActive = bNewIsActive;
	MARK_PROPERTY_DIRTY_FROM_NAME(UActorComponent, bIsActive, this);
}

bool UActorComponent::OwnerNeedsInitialization() const
{
	AActor* Owner = GetOwner();
	return Owner && Owner->HasAnyFlags(RF_NeedInitialization);
}

bool UActorComponent::NeedsInitialization() const
{
	return HasAnyFlags(RF_NeedInitialization);
}

#if WITH_EDITOR
TFunction<bool(const UActorComponent*)> GIsComponentSelectedInEditor;
#endif

bool UActorComponent::CanBeHLODRelevant(const UActorComponent* ActorComponent)
{
	if (!IsValid(ActorComponent))
	{
		return false;
	}

	if (ActorComponent->HasAnyFlags(RF_Transient)
#if WITH_EDITOR 
		&& !(ActorComponent->GetOwner() && ActorComponent->GetOwner()->IsInLevelInstance())			// Treat components in LI as HLOD relevant for the sake of visualisation modes
#endif
		)
	{
		return false;
	}

#if WITH_EDITORONLY_DATA
	if (ActorComponent->IsEditorOnly())
	{
		return false;
	}

	if (ActorComponent->IsVisualizationComponent())
	{
		return false;
	}
#endif

	return true;
}

#undef LOCTEXT_NAMESPACE

