// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ActorPrimitiveComponentInterface.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/MaterialInterface.h"
#include "Components/StaticMeshComponent.h"

bool FActorPrimitiveComponentInterface::IsRenderStateCreated() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->IsRenderStateCreated();
}

bool FActorPrimitiveComponentInterface::IsRenderStateDirty() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->IsRenderStateDirty();
}

bool FActorPrimitiveComponentInterface::ShouldCreateRenderState() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->ShouldCreateRenderState();
}

bool FActorPrimitiveComponentInterface::IsRegistered() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->IsRegistered();
}

bool FActorPrimitiveComponentInterface::IsUnreachable() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->IsUnreachable();
}

bool FActorPrimitiveComponentInterface::IsStaticMobility() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetMobility() == EComponentMobility::Static;
}

bool FActorPrimitiveComponentInterface::IsMipStreamingForced() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->IsMipStreamingForced();
}

UWorld* FActorPrimitiveComponentInterface::GetWorld() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetWorld();
}

FSceneInterface* FActorPrimitiveComponentInterface::GetScene() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetScene();
}

FPrimitiveSceneProxy* FActorPrimitiveComponentInterface::GetSceneProxy() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetSceneProxy();
}

void FActorPrimitiveComponentInterface::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	UPrimitiveComponent::GetPrimitiveComponent(this)->GetUsedMaterials(OutMaterials, bGetDebugMaterials);
}

void FActorPrimitiveComponentInterface::MarkRenderStateDirty()
{
	UPrimitiveComponent::GetPrimitiveComponent(this)->MarkRenderStateDirty();
}

void FActorPrimitiveComponentInterface::OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex)
{
	UPrimitiveComponent::GetPrimitiveComponent(this)->OnRenderAssetFirstLodChange(RenderAsset, FirstLodIndex);
}

void FActorPrimitiveComponentInterface::DestroyRenderState()
{
	UPrimitiveComponent::GetPrimitiveComponent(this)->DestroyRenderState_Concurrent();
}

void FActorPrimitiveComponentInterface::CreateRenderState(FRegisterComponentContext* Context)
{
	UPrimitiveComponent::GetPrimitiveComponent(this)->CreateRenderState_Concurrent(Context);
}

FString FActorPrimitiveComponentInterface::GetName() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetName();
}

FString FActorPrimitiveComponentInterface::GetFullName() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetFullName();
}

FTransform FActorPrimitiveComponentInterface::GetTransform() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetComponentTransform();
}

const FBoxSphereBounds& FActorPrimitiveComponentInterface::GetBounds() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->Bounds;
}

float FActorPrimitiveComponentInterface::GetLastRenderTimeOnScreen() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetLastRenderTimeOnScreen();
}

void FActorPrimitiveComponentInterface::GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetPrimitiveStats(PrimitiveStats);
}

UObject* FActorPrimitiveComponentInterface::GetUObject()
{
	return UPrimitiveComponent::GetPrimitiveComponent(this);
}

const UObject* FActorPrimitiveComponentInterface::GetUObject() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this);
}

UObject* FActorPrimitiveComponentInterface::GetOwner() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetOwner();
}

FString FActorPrimitiveComponentInterface::GetOwnerName() const
{
	const UPrimitiveComponent* Component = UPrimitiveComponent::GetPrimitiveComponent(this);

#if ACTOR_HAS_LABELS
	if (Component->GetOwner())
	{
		return Component->GetOwner()->GetActorNameOrLabel();
	}
#endif

	return Component->GetName();
}

FPrimitiveSceneProxy* FActorPrimitiveComponentInterface::CreateSceneProxy()
{
	UPrimitiveComponent* Component = UPrimitiveComponent::GetPrimitiveComponent(this);
	FPrimitiveSceneProxy* SceneProxy = Component->CreateSceneProxy();
	Component->AssignSceneProxy(SceneProxy);
	return SceneProxy;
}

FRenderAssetOwnerStreamingState& FActorPrimitiveComponentInterface::GetStreamingState() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetStreamingState();
}

ULevel* FActorPrimitiveComponentInterface::GetComponentLevel() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetComponentLevel();
}

IPrimitiveComponent* FActorPrimitiveComponentInterface::GetLODParentPrimitive() const
{
	UPrimitiveComponent* LodParent = UPrimitiveComponent::GetPrimitiveComponent(this)->GetLODParentPrimitive();
	return LodParent ? LodParent->GetPrimitiveComponentInterface() : nullptr;
}

float FActorPrimitiveComponentInterface::GetMinDrawDistance() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetMinDrawDistance();
}

float FActorPrimitiveComponentInterface::GetStreamingScale() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetStreamingScale();
}

UStreamableRenderAsset* FActorPrimitiveComponentInterface::GetStreamableNaniteAsset() const
{
	return UPrimitiveComponent::GetPrimitiveComponent(this)->GetStreamableNaniteAsset();
}

void FActorPrimitiveComponentInterface::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	UPrimitiveComponent::GetPrimitiveComponent(this)->GetStreamingRenderAssetInfo(LevelContext, OutStreamingRenderAssets);
}

void FActorPrimitiveComponentInterface::PrecachePSOs()
{
	UPrimitiveComponent::GetPrimitiveComponent(this)->PrecachePSOs();
}

#if WITH_EDITOR
HHitProxy* FActorPrimitiveComponentInterface::CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex)
{
	UPrimitiveComponent* Component = UPrimitiveComponent::GetPrimitiveComponent(this);
	return Component->CreateMeshHitProxy(SectionIndex, MaterialIndex);
}
#endif
