// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ActorSkinnedMeshComponentInterface.h"
#include "Components/SkinnedMeshComponent.h"
#include "ComponentReregisterContext.h"

#if WITH_EDITOR
void FActorSkinnedMeshComponentInterface::PostAssetCompilation()
{
	USkinnedMeshComponent::GetSkinnedMeshComponent(this)->PostAssetCompilation();
}

void FActorSkinnedMeshComponentInterface::PostSkeletalHierarchyChange()
{
	FComponentReregisterContext Context(USkinnedMeshComponent::GetSkinnedMeshComponent(this));
}
#endif

USkinnedAsset* FActorSkinnedMeshComponentInterface::GetSkinnedAsset() const
{
	return USkinnedMeshComponent::GetSkinnedMeshComponent(this)->GetSkinnedAsset();
}

IPrimitiveComponent* FActorSkinnedMeshComponentInterface::GetPrimitiveComponentInterface()
{
	return USkinnedMeshComponent::GetSkinnedMeshComponent(this)->GetPrimitiveComponentInterface();
}
