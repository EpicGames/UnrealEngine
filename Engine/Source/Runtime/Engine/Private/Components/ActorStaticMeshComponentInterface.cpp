// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ActorStaticMeshComponentInterface.h"
#include "Components/StaticMeshComponent.h"

#if WITH_EDITOR
void FActorStaticMeshComponentInterface::OnMeshRebuild(bool bRenderDataChanged)
{
	UStaticMeshComponent::GetStaticMeshComponent(this)->OnMeshRebuild(bRenderDataChanged);
}

void FActorStaticMeshComponentInterface::PreStaticMeshCompilation()
{
	UStaticMeshComponent::GetStaticMeshComponent(this)->PreStaticMeshCompilation();
}

void FActorStaticMeshComponentInterface::PostStaticMeshCompilation()
{
	UStaticMeshComponent::GetStaticMeshComponent(this)->PostStaticMeshCompilation();
}
#endif

UStaticMesh* FActorStaticMeshComponentInterface::GetStaticMesh() const
{
	return UStaticMeshComponent::GetStaticMeshComponent(this)->GetStaticMesh();
}

IPrimitiveComponent* FActorStaticMeshComponentInterface::GetPrimitiveComponentInterface()
{
	return UStaticMeshComponent::GetStaticMeshComponent(this)->GetPrimitiveComponentInterface();
}
