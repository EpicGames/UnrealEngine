// Copyright Epic Games, Inc. All Rights Reserved.
// ApplicationLifecycleComponent.cpp: Component 
// handle receiving notifications from the OS about application state (activated, suspended, termination, etc)

#include "Components/ApplicationLifecycleComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ApplicationLifecycleComponent)

UApplicationLifecycleComponent::UApplicationLifecycleComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UApplicationLifecycleComponent::OnRegister()
{
	Super::OnRegister();

	FCoreDelegates::ApplicationWillDeactivateDelegate.AddUObject(this, &UApplicationLifecycleComponent::ApplicationWillDeactivateDelegate_Handler);
	FCoreDelegates::ApplicationHasReactivatedDelegate.AddUObject(this, &UApplicationLifecycleComponent::ApplicationHasReactivatedDelegate_Handler);
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddUObject(this, &UApplicationLifecycleComponent::ApplicationWillEnterBackgroundDelegate_Handler);
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddUObject(this, &UApplicationLifecycleComponent::ApplicationHasEnteredForegroundDelegate_Handler);
	FCoreDelegates::GetApplicationWillTerminateDelegate().AddUObject(this, &UApplicationLifecycleComponent::ApplicationWillTerminateDelegate_Handler);
	FCoreDelegates::ApplicationShouldUnloadResourcesDelegate.AddUObject(this, &UApplicationLifecycleComponent::ApplicationShouldUnloadResourcesDelegate_Handler);
	FCoreDelegates::ApplicationReceivedStartupArgumentsDelegate.AddUObject(this, &UApplicationLifecycleComponent::ApplicationReceivedStartupArgumentsDelegate_Handler);

	FCoreDelegates::OnTemperatureChange.AddUObject(this, &UApplicationLifecycleComponent::OnTemperatureChangeDelegate_Handler);
	FCoreDelegates::OnLowPowerMode.AddUObject(this, &UApplicationLifecycleComponent::OnLowPowerModeDelegate_Handler);
}

void UApplicationLifecycleComponent::OnUnregister()
{
	Super::OnUnregister();
	
 	FCoreDelegates::ApplicationWillDeactivateDelegate.RemoveAll(this);
 	FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
 	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);
 	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.RemoveAll(this);
 	FCoreDelegates::GetApplicationWillTerminateDelegate().RemoveAll(this);
 	FCoreDelegates::ApplicationShouldUnloadResourcesDelegate.RemoveAll(this);
 	FCoreDelegates::ApplicationReceivedStartupArgumentsDelegate.RemoveAll(this);
	FCoreDelegates::OnTemperatureChange.RemoveAll(this);
	FCoreDelegates::OnLowPowerMode.RemoveAll(this);
}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/IsConst.h"
#endif
