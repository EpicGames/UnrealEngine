// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ArrowComponent.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/Engine.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "Engine/CollisionProfile.h"
#include "SceneInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMeshBuilder.h"
#include "SceneView.h"
#include "UObject/UObjectIterator.h"
#include "StaticMeshResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ArrowComponent)

#define DEFAULT_SCREEN_SIZE	(0.0025f)
#define ARROW_SCALE			(80.0f)
#define ARROW_RADIUS_FACTOR	(0.03f)
#define ARROW_HEAD_FACTOR	(0.2f)
#define ARROW_HEAD_ANGLE	(20.f)

#if WITH_EDITORONLY_DATA
float UArrowComponent::EditorScale = 1.0f;
#endif

/** Represents a UArrowComponent to the scene manager. */
class FArrowSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FArrowSceneProxy(UArrowComponent* Component)
		: FPrimitiveSceneProxy(Component)
		, VertexFactory(GetScene().GetFeatureLevel(), "FArrowSceneProxy")
		, ArrowColor(Component->ArrowColor)
		, ArrowSize(Component->ArrowSize)
		, ArrowLength(Component->ArrowLength)
		, bIsScreenSizeScaled(Component->bIsScreenSizeScaled)
		, ScreenSize(Component->ScreenSize)
#if WITH_EDITORONLY_DATA
		, bLightAttachment(Component->bLightAttachment)
		, bTreatAsASprite(Component->bTreatAsASprite)
		, bUseInEditorScaling(Component->bUseInEditorScaling)
		, EditorScale(Component->EditorScale)
#endif
	{
		bWillEverBeLit = false;
#if WITH_EDITOR
		// If in the editor, extract the sprite category from the component
		if ( GIsEditor )
		{
			SpriteCategoryIndex = GEngine->GetSpriteCategoryIndex( Component->SpriteInfo.Category );
		}
#endif	//WITH_EDITOR

		const float HeadAngle = FMath::DegreesToRadians(ARROW_HEAD_ANGLE);
		const float DefaultLength = ArrowSize * ARROW_SCALE;
		const float TotalLength = ArrowSize * ArrowLength;
		const float HeadLength = DefaultLength * ARROW_HEAD_FACTOR;
		const float ShaftRadius = DefaultLength * ARROW_RADIUS_FACTOR;
		const float ShaftLength = (TotalLength - HeadLength * 0.5); // 10% overlap between shaft and head
		const FVector ShaftCenter = FVector(0.5f * ShaftLength, 0, 0);

		TArray<FDynamicMeshVertex> OutVerts;
		BuildConeVerts(HeadAngle, HeadAngle, -HeadLength, TotalLength, 32, OutVerts, IndexBuffer.Indices);
		BuildCylinderVerts(ShaftCenter, FVector(0,0,1), FVector(0,1,0), FVector(1,0,0), ShaftRadius, 0.5f * ShaftLength, 16, OutVerts, IndexBuffer.Indices);

		VertexBuffers.InitFromDynamicVertex(&VertexFactory, OutVerts);

		// Enqueue initialization of render resource
		BeginInitResource(&IndexBuffer);
	}

	virtual ~FArrowSceneProxy()
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	// FPrimitiveSceneProxy interface.
	
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER( STAT_ArrowSceneProxy_DrawDynamicElements );

		FMatrix EffectiveLocalToWorld;
#if WITH_EDITOR
		if (bLightAttachment)
		{
			EffectiveLocalToWorld = GetLocalToWorld().GetMatrixWithoutScale();
		}
		else
#endif	//WITH_EDITOR
		{
			EffectiveLocalToWorld = GetLocalToWorld();
		}

		auto ArrowMaterialRenderProxy = new FColoredMaterialRenderProxy(
			GEngine->ArrowMaterial->GetRenderProxy(),
			ArrowColor,
			"GizmoColor"
			);

		Collector.RegisterOneFrameMaterialProxy(ArrowMaterialRenderProxy);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];

				// Calculate the view-dependent scaling factor.
				float ViewScale = 1.0f;
				if (bIsScreenSizeScaled && (View->ViewMatrices.GetViewToClip().M[3][3] != 1.0f))
				{
					const float ZoomFactor = FMath::Min<float>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
					if (ZoomFactor != 0.0f)
					{
						// Note: we can't just ignore the perspective scaling here if the object's origin is behind the camera, so preserve the scale minus its sign.
						const float Radius = FMath::Abs(View->WorldToScreen(Origin).W * (ScreenSize / ZoomFactor));
						if (Radius < 1.0f)
						{
							ViewScale *= Radius;
						}
					}
				}

		#if WITH_EDITORONLY_DATA
				ViewScale *= EditorScale;
		#endif

				// Draw the mesh.
				FMeshBatch& Mesh = Collector.AllocateMesh();
				FMeshBatchElement& BatchElement = Mesh.Elements[0];
				BatchElement.IndexBuffer = &IndexBuffer;
				Mesh.bWireframe = false;
				Mesh.VertexFactory = &VertexFactory;
				Mesh.MaterialRenderProxy = ArrowMaterialRenderProxy;

				FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
				DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), FScaleMatrix(ViewScale) * EffectiveLocalToWorld, FScaleMatrix(ViewScale) * EffectiveLocalToWorld, GetBounds(), GetLocalBounds(), true, false, AlwaysHasVelocity());
				BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

				BatchElement.FirstIndex = 0;
				BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
				BatchElement.MinVertexIndex = 0;
				BatchElement.MaxVertexIndex = VertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;
				Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
				Mesh.Type = PT_TriangleList;
				Mesh.DepthPriorityGroup = SDPG_World;
				Mesh.bCanApplyViewModeOverrides = false;
				Collector.AddMesh(ViewIndex, Mesh);
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View) && (View->Family->EngineShowFlags.BillboardSprites);
		Result.bDynamicRelevance = true;
#if WITH_EDITOR
		if (bTreatAsASprite)
		{
			if ( GIsEditor && SpriteCategoryIndex != INDEX_NONE && SpriteCategoryIndex < View->SpriteCategoryVisibility.Num() && !View->SpriteCategoryVisibility[ SpriteCategoryIndex ] )
			{
				Result.bDrawRelevance = false;
			}
		}
#endif
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override
	{
		Origin = GetLocalToWorld().GetOrigin();
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

private:
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
	FLocalVertexFactory VertexFactory;

	FVector Origin;
	FColor ArrowColor;
	float ArrowSize;
	float ArrowLength;
	bool bIsScreenSizeScaled;
	float ScreenSize;
#if WITH_EDITORONLY_DATA
	bool bLightAttachment;
	bool bTreatAsASprite;
	int32 SpriteCategoryIndex;
	bool bUseInEditorScaling;
	float EditorScale;
#endif // #if WITH_EDITORONLY_DATA
};

UArrowComponent::UArrowComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName ID_Misc;
		FText NAME_Misc;
		FConstructorStatics()
			: ID_Misc(TEXT("Misc"))
			, NAME_Misc(NSLOCTEXT( "SpriteCategory", "Misc", "Misc" ))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	ArrowColor = FColor(255, 0, 0, 255);

	ArrowSize = 1.0f;
	ArrowLength = ARROW_SCALE;
	bHiddenInGame = true;
	bUseEditorCompositing = true;
	SetGenerateOverlapEvents(false);
	bIsScreenSizeScaled = false;
	ScreenSize = DEFAULT_SCREEN_SIZE;
#if WITH_EDITORONLY_DATA
	SpriteInfo.Category = ConstructorStatics.ID_Misc;
	SpriteInfo.DisplayName = ConstructorStatics.NAME_Misc;
	bLightAttachment = false;
	bUseInEditorScaling = true;
#endif // WITH_EDITORONLY_DATA
}

FPrimitiveSceneProxy* UArrowComponent::CreateSceneProxy()
{
	return new FArrowSceneProxy(this);
}

#if WITH_EDITOR
bool UArrowComponent::ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	// Arrow components not treated as 'selectable' in editor
	return false;
}

bool UArrowComponent::ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	// Arrow components not treated as 'selectable' in editor
	return false;
}
#endif


FBoxSphereBounds UArrowComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FBox(FVector(0,-ARROW_SCALE,-ARROW_SCALE),FVector(ArrowSize * ArrowLength * 3.0f,ARROW_SCALE,ARROW_SCALE))).TransformBy(LocalToWorld);
}

void UArrowComponent::SetArrowColor(FLinearColor NewColor)
{
	SetArrowFColor(NewColor.ToFColor(true));
}

void UArrowComponent::SetArrowFColor(FColor NewColor)
{
	ArrowColor = NewColor;
	MarkRenderStateDirty();
}

void UArrowComponent::SetArrowSize(float NewSize)
{
	ArrowSize = NewSize;
	MarkRenderStateDirty();
}

void UArrowComponent::SetArrowLength(float NewLength)
{
	ArrowLength = NewLength;
	MarkRenderStateDirty();
}

void UArrowComponent::SetScreenSize(float NewScreenSize)
{
	ScreenSize = NewScreenSize;
	MarkRenderStateDirty();
}

void UArrowComponent::SetIsScreenSizeScaled(bool bNewValue)
{
	bIsScreenSizeScaled = bNewValue;
	MarkRenderStateDirty();
}

void UArrowComponent::SetTreatAsASprite(bool bNewValue)
{
	bTreatAsASprite = bNewValue;
	MarkRenderStateDirty();
}

void UArrowComponent::SetUseInEditorScaling(bool bNewValue)
{
#if WITH_EDITORONLY_DATA
	bUseInEditorScaling = bNewValue;
	MarkRenderStateDirty();
#else
	const bool bCallOutsideOf_WithEditorOnlyData = false;
	ensure(bCallOutsideOf_WithEditorOnlyData);
#endif
}

#if WITH_EDITORONLY_DATA
void UArrowComponent::SetEditorScale(float InEditorScale)
{
	EditorScale = InEditorScale;
	for(TObjectIterator<UArrowComponent> It; It; ++It)
	{
		It->MarkRenderStateDirty();
	}
}
#endif

