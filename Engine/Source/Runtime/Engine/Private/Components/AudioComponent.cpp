// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/AudioComponent.h"

#include "Algo/AllOf.h"
#include "Algo/Transform.h"
#include "Audio/ActorSoundParameterInterface.h"
#include "Audio/AudioTraceUtil.h"
#include "AudioDevice.h"
#include "Components/BillboardComponent.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/ScopeLock.h"
#include "Quartz/AudioMixerClockHandle.h"
#include "Quartz/QuartzSubsystem.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeAttenuation.h"
#include "Stats/StatsTrace.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/ICookInfo.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioComponent)

DECLARE_CYCLE_STAT(TEXT("AudioComponent Play"), STAT_AudioComp_Play, STATGROUP_Audio);

static float BakedAnalysisTimeShiftCVar = 0.0f;
FAutoConsoleVariableRef CVarBackedAnalysisTimeShift(
	TEXT("au.AnalysisTimeShift"),
	BakedAnalysisTimeShiftCVar,
	TEXT("Shifts the timeline for baked analysis playback.\n")
	TEXT("Value: The time in seconds to shift the timeline."),
	ECVF_Default);

static int32 PrimeSoundOnAudioComponentSpawnCVar = 0;
FAutoConsoleVariableRef CVarPrimeSoundOnAudioComponentSpawn(
	TEXT("au.streamcaching.PrimeSoundOnAudioComponents"),
	PrimeSoundOnAudioComponentSpawnCVar,
	TEXT("When set to 1, automatically primes a USoundBase when a UAudioComponent is spawned with that sound, or when UAudioComponent::SetSound is called.\n"),
	ECVF_Default);

//CVar for how long voiceslots should be taken up when queuing sounds
static int32 TimeToTakeUpVoiceSlotCVar = int32(EQuartzCommandQuantization::HalfNote);
FAutoConsoleVariableRef CVarTimeToTakeUpVoiceSlot(
	TEXT("au.Quartz.TimeToTakeUpVoiceSlot"),
	TimeToTakeUpVoiceSlotCVar,
	TEXT("TheEQuartzCommandQuantization type (default: EQuartzCommandQuantization::EighthNote) before playing that a queued sound should take up a voice slot for\n")
	TEXT("Value: The EQuartzCommandQuantization index of the desired duration"),
	ECVF_Default);

//CVar to disable the gamethread-side caching of play requests to minimize time spent rendering a silent voice before the Quartz Deadline
static int32 bAlwaysTakeVoiceSlotCVar = 1;
FAutoConsoleVariableRef bCVarAlwaysTakeVoiceSlot(
	TEXT("au.Quartz.bAlwaysTakeVoiceSlot"),
	bAlwaysTakeVoiceSlotCVar,
	TEXT("Always take voice slot immediately without trying to cache the request on the component\n")
	TEXT("default = 1: always forward the request to the audio engine immediately. - 0: attempt to cache play requests on the component until closer to the deadline."),
	ECVF_Default);


static int32 WorldlessGetAudioTimeBehaviorCVar = 0;
FAutoConsoleVariableRef CVarWorldlessGetAudioTimeBehavior(
	TEXT("au.WorldlessGetAudioTimeBehavior"),
	WorldlessGetAudioTimeBehaviorCVar,
	TEXT("Determines the return value of GetAudioTime when an audio component does not belong to a world.\n")
	TEXT("0: 0.f (default), 1: Application's CurrentTime"),
	ECVF_Default);

uint64 UAudioComponent::AudioComponentIDCounter = 0;
TMap<uint64, UAudioComponent*> UAudioComponent::AudioIDToComponentMap;
FTransactionallySafeCriticalSection UAudioComponent::AudioIDToComponentMapLock;

#if WITH_EDITORONLY_DATA
static const TCHAR* GAudioSpriteAssetNameAutoActivate = TEXT("/Engine/EditorResources/AudioIcons/S_AudioComponent_AutoActivate.S_AudioComponent_AutoActivate");
static const TCHAR* GAudioSpriteAssetName = TEXT("/Engine/EditorResources/AudioIcons/S_AudioComponent.S_AudioComponent");
#endif

namespace AudioComponentPrivate
{
	struct FModulatorWeakAccessor
	{
	private:
		TSet<TWeakObjectPtr<USoundModulatorBase>> WeakModulators;

	public:
		FModulatorWeakAccessor(const TSet<TObjectPtr<USoundModulatorBase>>& Modulators)
		{
			Algo::Transform(Modulators, WeakModulators, [](const TObjectPtr<USoundModulatorBase>& Obj)
			{
				return TWeakObjectPtr<USoundModulatorBase>(Obj.Get());
			});
		}

		bool TryGetSafe(TSet<TObjectPtr<USoundModulatorBase>>& OutModulators) const
		{
			OutModulators.Reset();

			auto IsValid = [](const TWeakObjectPtr<USoundModulatorBase>& WeakPtr) { return WeakPtr.IsValid(); };
			if (!Algo::AllOf(WeakModulators, IsValid))
			{
				return false;
			}

			auto AsObjectPtr = [](const TWeakObjectPtr<USoundModulatorBase>& WeakPtr) { return WeakPtr.Get(); };
			Algo::Transform(WeakModulators, OutModulators, AsObjectPtr);
			return true;
		}
	};
} // namespace AudioComponentPrivate

UInitialActiveSoundParams::UInitialActiveSoundParams(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAudioComponent::UAudioComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseAttachParentBound = true; // Avoid CalcBounds() when transform changes.
	bAutoDestroy = false;
	bAutoManageAttachment = false;
	bAutoActivate = true;
	bAllowAnyoneToDestroyMe = true;
	bAllowSpatialization = true;
	bStopWhenOwnerDestroyed = true;
	bNeverNeedsRenderUpdate = true;
	bWantsOnUpdateTransform = true;
#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif
	VolumeMultiplier = 1.f;
	bOverridePriority = false;
	bOverrideSubtitlePriority = false;
	bCanPlayMultipleInstances = false;
	bDisableParameterUpdatesWhilePlaying = false;
	bIsPreviewSound = false;
#if WITH_EDITORONLY_DATA
	bIsDryPreviewPlayback = false;
#endif //WITH_EDITORONLY_DATA
	bIsPaused = false;

	Priority = 1.f;
	SubtitlePriority = DEFAULT_SUBTITLE_PRIORITY;
	PitchMultiplier = 1.f;
	VolumeModulationMin = 1.f;
	VolumeModulationMax = 1.f;
	PitchModulationMin = 1.f;
	PitchModulationMax = 1.f;
	bEnableLowPassFilter = false;
	bEnableHighPassFilter = false;
	LowPassFilterFrequency = MAX_FILTER_FREQUENCY;
	HighPassFilterFrequency = MIN_FILTER_FREQUENCY;
	OcclusionCheckInterval = 0.1f;
	ActiveCount = 0;

	EnvelopeFollowerAttackTime = 10;
	EnvelopeFollowerReleaseTime = 100;

	AudioDeviceID = INDEX_NONE;

	RandomStream.Initialize(FApp::bUseFixedSeed ? GetFName() : NAME_None);

	{
		// TODO: Consider only putting played/active components in to the map
		UE::TScopeLock Lock(AudioIDToComponentMapLock);
		AudioComponentID = ++AudioComponentIDCounter;
		AudioIDToComponentMap.Add(AudioComponentID, this);
	}
}

UAudioComponent* UAudioComponent::GetAudioComponentFromID(uint64 AudioComponentID)
{
	// Although we should be in the game thread when calling this function, async loading makes it possible/common for these
	// components to be constructed outside of the game thread. This means we need a lock around anything that deals with the
	// AudioIDToComponentMap.
	UE::TScopeLock Lock(AudioIDToComponentMapLock);
	return AudioIDToComponentMap.FindRef(AudioComponentID);
}

void UAudioComponent::BeginDestroy()
{
	if (IsActive() && Sound && Sound->IsLooping())
	{
		UE_LOGF(LogAudio, Verbose, "Audio Component is being destroyed prior to stopping looping sound '%ls' directly.", *Sound->GetFullName());
		Stop();
	}

	ResetParameters();

	{
		UE::TScopeLock Lock(AudioIDToComponentMapLock);
		AudioIDToComponentMap.Remove(AudioComponentID);
	}

	Super::BeginDestroy();
}

FString UAudioComponent::GetDetailedInfoInternal() const
{
	FString Result;

	if (Sound != nullptr)
	{
		Result = Sound->GetPathName(nullptr);
	}
	else
	{
		Result = TEXT( "No_Sound" );
	}

	return Result;
}

void UAudioComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::ChangeAudioComponentOverrideSubtitlePriorityDefault)
	{
		// Since the default for overriding the priority changed delta serialize would not have written out anything for true, so if they've changed
		// the priority we'll assume they wanted true, otherwise, we'll leave it with the new false default
		if (SubtitlePriority != DEFAULT_SUBTITLE_PRIORITY)
		{
			bOverrideSubtitlePriority = true;
		}
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (ConcurrencySettings_DEPRECATED != nullptr)
		{
			ConcurrencySet.Add(ConcurrencySettings_DEPRECATED);
			ConcurrencySettings_DEPRECATED = nullptr;
		}

		ModulationRouting.VersionModulators();
	}
	if (Ar.IsSaving() && Ar.IsObjectReferenceCollector() && !Ar.IsCooking())
	{
		FSoftObjectPathSerializationScope EditorOnlyScope(ESoftObjectPathCollectType::EditorOnlyCollect);
		FSoftObjectPath SpriteAssets[]{ FSoftObjectPath(GAudioSpriteAssetNameAutoActivate), FSoftObjectPath(GAudioSpriteAssetName) };
		for (FSoftObjectPath& SpriteAsset : SpriteAssets)
		{
			Ar << SpriteAsset;
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UAudioComponent::PostLoad()
{
#if WITH_EDITORONLY_DATA
	const FPackageFileVersion LinkerUEVersion = GetLinkerUEVersion();

	// Translate the old HighFrequencyGainMultiplier value to the new LowPassFilterFrequency value
	if (LinkerUEVersion < VER_UE4_USE_LOW_PASS_FILTER_FREQ)
	{
		if (HighFrequencyGainMultiplier_DEPRECATED > 0.0f &&  HighFrequencyGainMultiplier_DEPRECATED < 1.0f)
		{
			bEnableLowPassFilter = true;

			// This seems like it wouldn't make sense, but the original implementation for HighFrequencyGainMultiplier (a number between 0.0 and 1.0).
			// In earlier versions, this was *not* used as a high frequency gain, but instead converted to a frequency value between 0.0 and 6000.0
			// then "converted" to a radian frequency value using an equation taken from XAudio2 documentation. To recover
			// the original intended frequency (approximately), we'll run it through that equation, then scale radian value by the max filter frequency.

			float FilterConstant = 2.0f * FMath::Sin(UE_PI * 6000.0f * HighFrequencyGainMultiplier_DEPRECATED / 48000);
			LowPassFilterFrequency = FilterConstant * MAX_FILTER_FREQUENCY;
		}
	}
#endif

	if (PrimeSoundOnAudioComponentSpawnCVar && Sound)
	{
		UGameplayStatics::PrimeSound(Sound);
	}

	Super::PostLoad();
}

void UAudioComponent::OnRegister()
{
	if (bAutoManageAttachment && !IsActive())
	{
		// Detach from current parent, we are supposed to wait for activation.
		if (GetAttachParent())
		{
			// If no auto attach parent override, use the current parent when we activate
			if (!AutoAttachParent.IsValid())
			{
				AutoAttachParent = GetAttachParent();
			}
			// If no auto attach socket override, use current socket when we activate
			if (AutoAttachSocketName == NAME_None)
			{
				AutoAttachSocketName = GetAttachSocketName();
			}

			// If in a game world, detach now if necessary. Activation will cause auto-attachment.
			const UWorld* World = GetWorld();
			if (World->IsGameWorld())
			{
				// Prevent attachment before Super::OnRegister() tries to attach us, since we only attach when activated.
				if (GetAttachParent()->GetAttachChildren().Contains(this))
				{
					// Only detach if we are not about to auto attach to the same target, that would be wasteful.
					if (!bAutoActivate || (AutoAttachLocationRule != EAttachmentRule::KeepRelative && AutoAttachRotationRule != EAttachmentRule::KeepRelative && AutoAttachScaleRule != EAttachmentRule::KeepRelative) || (AutoAttachSocketName != GetAttachSocketName()) || (AutoAttachParent != GetAttachParent()))
					{
						DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, /*bCallModify=*/ false));
					}
				}
				else
				{
					SetupAttachment(nullptr, NAME_None);
				}
			}
		}

		SavedAutoAttachRelativeLocation = GetRelativeLocation();
		SavedAutoAttachRelativeRotation = GetRelativeRotation();
		SavedAutoAttachRelativeScale3D = GetRelativeScale3D();
	}

	Super::OnRegister();

	#if WITH_EDITORONLY_DATA
	UpdateSpriteTexture();
	#endif
}

void UAudioComponent::OnUnregister()
{
	// Route OnUnregister event.
	Super::OnUnregister();

	// Don't stop audio and clean up component if owner has been destroyed (default behaviour). This function gets
	// called from AActor::ClearComponents when an actor gets destroyed which is not usually what we want for one-
	// shot sounds.
	AActor* Owner = GetOwner();
	if (!Owner || bStopWhenOwnerDestroyed)
	{
		Stop();
	}
}

void UAudioComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
	QuartzUnsubscribe();
}

const UObject* UAudioComponent::AdditionalStatObject() const
{
	return Sound;
}

void UAudioComponent::SetSound(USoundBase* NewSound)
{
	const bool bPlay = IsPlaying();

	// If this is an auto destroy component we need to prevent it from being auto-destroyed since we're really just restarting it
	const bool bWasAutoDestroy = bAutoDestroy;
	bAutoDestroy = false;
	//Only stop the existing sound if we are limited to one sound per component
	if (!bCanPlayMultipleInstances)
	{
		Stop();
	}
	bAutoDestroy = bWasAutoDestroy;

	Sound = NewSound;

	if (PrimeSoundOnAudioComponentSpawnCVar && Sound)
	{
		UGameplayStatics::PrimeSound(Sound);
	}

	if (bPlay && !bCanPlayMultipleInstances)
	{
		Play();
	}
}

bool UAudioComponent::IsReadyForOwnerToAutoDestroy() const
{
	return !IsPlaying();
}

void UAudioComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	Super::OnUpdateTransform(UpdateTransformFlags, Teleport);

	if (bPreviewComponent)
	{
		return;
	}

	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		if (IsActive())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.UpdateAudioComponentTransform"), STAT_AudioUpdateComponentTransform, STATGROUP_AudioThreadCommands);

			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [NewTransform = GetComponentTransform()](FActiveSound& ActiveSound)
			{
				ActiveSound.Transform = NewTransform;
			});
		}
	}
};

void UAudioComponent::BroadcastPlayState()
{
	if (OnAudioPlayStateChanged.IsBound())
	{
		OnAudioPlayStateChanged.Broadcast(GetPlayState());
	}

	if (OnAudioPlayStateChangedNative.IsBound())
	{
		OnAudioPlayStateChangedNative.Broadcast(this, GetPlayState());
	}
}

float UAudioComponent::GetAudioTimeSeconds() const
{
	if (UWorld* World = GetWorld())
	{
		return World->GetAudioTimeSeconds();
	}

	if (WorldlessGetAudioTimeBehaviorCVar)
	{
		return FApp::GetCurrentTime();
	}

	return 0.f;
}

FBoxSphereBounds UAudioComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const USceneComponent* UseAutoParent = (bAutoManageAttachment && GetAttachParent() == nullptr) ? AutoAttachParent.Get() : nullptr;
	if (UseAutoParent)
	{
		// We use auto attachment but have detached, don't use our own bogus bounds (we're off near 0,0,0), use the usual parent's bounds.
		return UseAutoParent->Bounds;
	}

	return Super::CalcBounds(LocalToWorld);
}

void UAudioComponent::CancelAutoAttachment(bool bDetachFromParent, const UWorld* MyWorld)
{
	if (bAutoManageAttachment && MyWorld && MyWorld->IsGameWorld())
	{
		if (bDidAutoAttach)
		{
			// Restore relative transform from before attachment. Actual transform will be updated as part of DetachFromParent().
			SetRelativeLocation_Direct(SavedAutoAttachRelativeLocation);
			SetRelativeRotation_Direct(SavedAutoAttachRelativeRotation);
			SetRelativeScale3D_Direct(SavedAutoAttachRelativeScale3D);
			bDidAutoAttach = false;
		}

		if (bDetachFromParent)
		{
			DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
		}
	}
}

bool UAudioComponent::IsInAudibleRange(float* OutMaxDistance) const
{
	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return false;
	}

	float MaxDistance = 0.0f;
	float FocusFactor = 0.0f;
	const FVector Location = GetComponentTransform().GetLocation();
	const FSoundAttenuationSettings* AttenuationSettingsToApply = bAllowSpatialization ? GetAttenuationSettingsToApply() : nullptr;
	AudioDevice->GetMaxDistanceAndFocusFactor(Sound, GetWorld(), Location, AttenuationSettingsToApply, MaxDistance, FocusFactor);

	if (OutMaxDistance)
	{
		*OutMaxDistance = MaxDistance;
	}

	return AudioDevice->SoundIsAudible(Sound, GetWorld(), Location, AttenuationSettingsToApply, MaxDistance, FocusFactor);
}

void UAudioComponent::Play(float StartTime)
{
	PlayInternalRequestData InternalRequestData;
	InternalRequestData.StartTime = StartTime;
	PlayInternal(InternalRequestData);
}

void UAudioComponent::ProcessCommand(const Audio::FQuartzQueueCommandData& InQueueCommandData)
{
	PlayQueuedQuantizedInternal(GetWorld(), InQueueCommandData.AudioComponentCommandInfo);
}

void UAudioComponent::PlayQuantized(
	  const UObject* WorldContextObject
	, UPARAM(ref) UQuartzClockHandle*& InClockHandle
	, UPARAM(ref) FQuartzQuantizationBoundary& InQuantizationBoundary
	, const FOnQuartzCommandEventBP& InDelegate
	, float InStartTime
	, float InFadeInDuration
	, float InFadeVolumeLevel
	, EAudioFaderCurve InFadeCurve)
{
	//Initialize the tickable object portion of the Audio Component, if it hasn't been initialized already
	if (!bAlwaysTakeVoiceSlotCVar && !FQuartzTickableObject::IsInitialized())
	{
		// if the WorldContextObject is null, attempt to fallback on our world
		// (call to Init() will ensure on a valid world)
		Init(WorldContextObject ? WorldContextObject->GetWorld() : GetWorld());
		check(FQuartzTickableObject::IsInitialized());
	}

	int32 TimeCVarVal = FMath::Clamp(CVarTimeToTakeUpVoiceSlot->GetInt(), 0, (int32)EQuartzCommandQuantization::Count - 1);
	EQuartzCommandQuantization MinimumQuantization = EQuartzCommandQuantization(TimeCVarVal);

	// Make an anticipatory quantization boundary to try to avoid taking up a whole voice slot while waiting for a queued event
	FQuartzQuantizationBoundary AnticipationQuantizationBoundary = FQuartzQuantizationBoundary(MinimumQuantization, 1.0f, EQuarztQuantizationReference::CurrentTimeRelative, true);

	FAudioComponentCommandInfo NewComponentCommandInfo(FQuartzTickableObject::GetQuartzSubscriber(), AnticipationQuantizationBoundary);

	// And a new pending quartz command data
	FAudioComponentPendingQuartzCommandData AudioComponentQuartzCommandData
	{
		AnticipationQuantizationBoundary,
		InDelegate,
		InStartTime,
		InFadeInDuration,
		InFadeVolumeLevel,
		InFadeCurve,
		NewComponentCommandInfo.CommandID,
		InClockHandle
	};

	// Guard against a null clock handle
	if (InClockHandle == nullptr)
	{
		return;
	}

	// Decide if we need to queue up the command to play at a later date (and not take up a voice slot) or if we can execute the command immediately

	// Make new audio component command info struct
	Audio::FQuartzClockTickRate OutTickRate;
	InClockHandle->GetCurrentTickRate(WorldContextObject, OutTickRate);

	double NumFramesBeforeMinTime = OutTickRate.GetFramesPerDuration(MinimumQuantization);
	double NumFramesForDesiredTime = OutTickRate.GetFramesPerDuration(InQuantizationBoundary.Quantization) * InQuantizationBoundary.Multiplier;

	// If the desired quantization time is less than our min time, just execute immediately
	const bool bStealVoiceSlot = bAlwaysTakeVoiceSlotCVar || (NumFramesForDesiredTime <= NumFramesBeforeMinTime);
	const bool bClockIsNotRunning = !InClockHandle->IsClockRunning(WorldContextObject);
	const bool bCommandResetsClock = InQuantizationBoundary.bResetClockOnQueued;
	if (bStealVoiceSlot || bClockIsNotRunning || bCommandResetsClock)
	{
		AudioComponentQuartzCommandData.AnticipatoryBoundary = NewComponentCommandInfo.AnticipatoryBoundary = InQuantizationBoundary; // use the desired target boundary

		// Add to the list of pending data
		// todo: avoid this copy for OUR call to PlayQueuedQuantizedInternal()
		//		 (usually called by a Quartz command, so we follow the same data-caching pattern here)
		PendingQuartzCommandData.Add(AudioComponentQuartzCommandData);
		PlayQueuedQuantizedInternal(WorldContextObject, NewComponentCommandInfo);
	}
	else
	{
		// We need to make an "anticipatory" quantization boundary
		PendingQuartzCommandData.Add(AudioComponentQuartzCommandData); 
		InClockHandle->QueueQuantizedSound(WorldContextObject, InClockHandle, NewComponentCommandInfo, InDelegate, InQuantizationBoundary);
	}
}

void UAudioComponent::PlayQueuedQuantizedInternal(const UObject* WorldContextObject, FAudioComponentCommandInfo InCommandInfo)
{
	bool bFoundQuantizedCommand = false;
	bool bIsValidCommand = true;
	UQuartzClockHandle* Handle = nullptr; // will be retrieved from PendingQuartzCommandData

	// Retrieve the stored data
	for (int32 i = PendingQuartzCommandData.Num() - 1; i >= 0; --i)
	{		
		const FAudioComponentPendingQuartzCommandData& PendingData = PendingQuartzCommandData[i];

		// Find the pending command ID
		if (PendingData.CommandID == InCommandInfo.CommandID)
		{
			Handle = PendingData.ClockHandle.Get();

			// Retrieve the pending data and queue up the quartz command
			PlayInternalRequestData InternalRequestData;

			InternalRequestData.StartTime = PendingData.StartTime;
			InternalRequestData.FadeInDuration = PendingData.FadeDuration;
			InternalRequestData.FadeVolumeLevel = PendingData.FadeVolume;
			InternalRequestData.FadeCurve = PendingData.FadeCurve;

			// verify we have a sound to play
			if(Sound)
			{
				// confirm a valid handle
				if (Handle != nullptr)
				{
					InternalRequestData.QuantizedRequestData = UQuartzSubsystem::CreateRequestDataForSchedulePlaySound(Handle, PendingData.Delegate, PendingData.AnticipatoryBoundary);
					UGameplayStatics::PrimeSound(Sound);
				}

				// validate clock existence 
				if (!Handle)
				{
					UE_LOGF(LogAudioQuartz, Warning, "Attempting to play Quantized Sound without supplying a Clock Handle");
					bIsValidCommand = false;
				}
				else if (!Handle->DoesClockExist(WorldContextObject))
				{
					UE_LOGF(LogAudioQuartz, Warning, "Clock: '%ls' Does not exist! Cannot play quantized sound: '%ls'", *InternalRequestData.QuantizedRequestData.ClockName.ToString(), *this->Sound->GetName());
					bIsValidCommand = false;
				}

				// was the sound already stopped while we were caching it?
				if (PendingData.bHasBeenStoppedWhileQueued)
				{
					InternalRequestData.QuantizedRequestData.QuantizedCommandPtr->FailedToQueue(InternalRequestData.QuantizedRequestData);
					UE_LOGF(LogAudioQuartz, Verbose, "Sound (%ls) to be played (on Clock: %ls) was stopped before being evaluated to play internally", *InternalRequestData.QuantizedRequestData.ClockName.ToString(), *this->Sound->GetName());
					bIsValidCommand = false;
				}


				if (bIsValidCommand)
				{
					InternalRequestData.QuantizedRequestData.GameThreadSubscribers.Add(GetQuartzSubscriber());

					// Now play the quartz command
					PlayInternal(InternalRequestData);
				}
			}
			else // this component does not have a Sound to play
			{
				UE_LOGF(LogAudioQuartz, Warning, "Attempting to play Quantized Sound without supplying a valid Sound to play");
			}



			// remove the pending quartz command data from the audio component
			bFoundQuantizedCommand = true;
			PendingQuartzCommandData.RemoveAtSwap(i, EAllowShrinking::No);
			break;
		}
	}

	if (!bFoundQuantizedCommand)
	{
		UE_LOGF(LogAudioQuartz, Warning, "Failed to find command ID '%d' for sound '%ls'.",
			InCommandInfo.CommandID,
			*this->Sound->GetName());
	}
}

void UAudioComponent::PlayInternal(const PlayInternalRequestData& InPlayRequestData, USoundBase* InSoundOverride)
{
	SCOPE_CYCLE_COUNTER(STAT_AudioComp_Play);

	UWorld* World = GetWorld();
	const float AudioTimeSeconds = GetAudioTimeSeconds();
	USoundBase* SoundToPlay = InSoundOverride ? InSoundOverride : Sound.Get();
	if (SoundToPlay)
	{
		UE_LOGF(LogAudio, Verbose, "%g: Playing AudioComponent : '%ls' with Sound: '%ls'. OneShot?: '%ls'", AudioTimeSeconds, *GetFullName(), *SoundToPlay->GetName(), !SoundToPlay->IsLooping() ? TEXT("YES") : TEXT("NO"));
	}

	// Reset our fading out flag in case this is a reused audio component and we are replaying after previously fading out
	bIsFadingOut = false;

	// Stop sound if active & not set to play multiple instances, irrespective of whether or not a valid sound is set to play.
	const bool bIsSoundLooping = SoundToPlay && SoundToPlay->IsLooping();
	if (IsActive())
	{
		// Stop if this component is limited to one active sound that is not looping.
		if (!bCanPlayMultipleInstances || bIsSoundLooping)
		{
			// Prevent sound from being auto-destroyed since its just being restarted.
			bool bCurrentAutoDestroy = bAutoDestroy;
			bAutoDestroy = false;
			Stop();
			bAutoDestroy = bCurrentAutoDestroy;
		}
	}

	if (!SoundToPlay)
	{
		UE_LOGF(LogAudio, Verbose, "%g: AudioComponent '%ls' failed to play sound: No sound to play specified.'", World ? World->GetAudioTimeSeconds() : 0.0f, *GetFullName());
		return;
	}

	if (World && !World->bAllowAudioPlayback)
	{
		return;
	}

	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return;
	}

	// Store the time that this audio component played
	TimeAudioComponentPlayed = AudioTimeSeconds;
	FadeInTimeDuration = InPlayRequestData.FadeInDuration;

	// Auto attach if requested
	const bool bWasAutoAttached = bDidAutoAttach;
	bDidAutoAttach = false;
	if (bAutoManageAttachment && World && World->IsGameWorld())
	{
		USceneComponent* NewParent = AutoAttachParent.Get();
		if (NewParent)
		{
			const bool bAlreadyAttached = GetAttachParent() && (GetAttachParent() == NewParent) && (GetAttachSocketName() == AutoAttachSocketName) && GetAttachParent()->GetAttachChildren().Contains(this);
			if (!bAlreadyAttached)
			{
				bDidAutoAttach = bWasAutoAttached;
				CancelAutoAttachment(true, World);
				SavedAutoAttachRelativeLocation = GetRelativeLocation();
				SavedAutoAttachRelativeRotation = GetRelativeRotation();
				SavedAutoAttachRelativeScale3D = GetRelativeScale3D();
				AttachToComponent(NewParent, FAttachmentTransformRules(AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule, false), AutoAttachSocketName);
			}

			bDidAutoAttach = true;
		}
		else
		{
			CancelAutoAttachment(true, World);
		}
	}

	// Create / configure new ActiveSound
	const FSoundAttenuationSettings* AttenuationSettingsToApply = bAllowSpatialization ? GetAttenuationSettingsToApply() : nullptr;

	float MaxDistance = 0.0f;
	float FocusFactor = 1.0f;
	FVector Location = GetComponentTransform().GetLocation();

	AudioDevice->GetMaxDistanceAndFocusFactor(SoundToPlay, World, Location, AttenuationSettingsToApply, MaxDistance, FocusFactor);

	TSharedRef<FActiveSound> NewSharedActiveSound = MakeShared<FActiveSound>();
	FActiveSound& NewActiveSound = NewSharedActiveSound.Get();
	NewActiveSound.SetAudioComponent(*this);
	NewActiveSound.SetWorld(World);
	NewActiveSound.SetSound(SoundToPlay);
	NewActiveSound.SetSourceEffectChain(SourceEffectChain);
	NewActiveSound.SetSoundClass(SoundClassOverride);
	NewActiveSound.SetAttenuationSettingsAsset(GetAttenuationSettingsAsset());
	NewActiveSound.SetAttenuationSettingsOverride(bOverrideAttenuation);

	NewActiveSound.ConcurrencySet = ConcurrencySet;

	const float Volume = (VolumeModulationMax + ((VolumeModulationMin - VolumeModulationMax) * RandomStream.FRand())) * VolumeMultiplier;
	NewActiveSound.SetVolume(Volume);

	// The priority used for the active sound is the audio component's priority scaled with the sound's priority
	if (bOverridePriority)
	{
		NewActiveSound.Priority = Priority;
	}
	else
	{
		NewActiveSound.Priority = SoundToPlay->Priority;
	}

	const float Pitch = (PitchModulationMax + ((PitchModulationMin - PitchModulationMax) * RandomStream.FRand())) * PitchMultiplier;
	NewActiveSound.SetPitch(Pitch);

	NewActiveSound.bEnableLowPassFilter = bEnableLowPassFilter;
	NewActiveSound.LowPassFilterFrequency = LowPassFilterFrequency;
	NewActiveSound.bEnableHighPassFilter = bEnableHighPassFilter;
	NewActiveSound.HighPassFilterFrequency = HighPassFilterFrequency;
	NewActiveSound.OcclusionCheckInterval = OcclusionCheckInterval;
	NewActiveSound.RequestedStartTime = FMath::Max(0.f, InPlayRequestData.StartTime);

	if (bOverrideSubtitlePriority)
	{
		NewActiveSound.SubtitlePriority = SubtitlePriority;
	}
	else
	{
		NewActiveSound.SubtitlePriority = SoundToPlay->GetSubtitlePriority();
	}

	NewActiveSound.bShouldRemainActiveIfDropped = bShouldRemainActiveIfDropped;
	NewActiveSound.bHandleSubtitles = (!bSuppressSubtitles || OnQueueSubtitles.IsBound());
	NewActiveSound.bIgnoreForFlushing = bIgnoreForFlushing;

	NewActiveSound.bIsUISound = bIsUISound;
	NewActiveSound.bIsMusic = bIsMusic;
	NewActiveSound.bAlwaysPlay = bAlwaysPlay;
	NewActiveSound.bReverb = bReverb;
	NewActiveSound.bCenterChannelOnly = bCenterChannelOnly;
	NewActiveSound.bIsPreviewSound = bIsPreviewSound;
#if WITH_EDITORONLY_DATA
	NewActiveSound.bIsDryPreviewPlayback = bIsDryPreviewPlayback;
#endif //WITH_EDITORONLY_DATA
	NewActiveSound.bLocationDefined = !bPreviewComponent;
	NewActiveSound.bIsPaused = bIsPaused;

	if (NewActiveSound.bLocationDefined)
	{
		NewActiveSound.Transform = GetComponentTransform();
	}

	NewActiveSound.bAllowSpatialization = bAllowSpatialization;
	NewActiveSound.bHasAttenuationSettings = (AttenuationSettingsToApply != nullptr);
	if (NewActiveSound.bHasAttenuationSettings)
	{
		NewActiveSound.AttenuationSettings = *AttenuationSettingsToApply;
		NewActiveSound.FocusData.PriorityScale = AttenuationSettingsToApply->GetFocusPriorityScale(AudioDevice->GetGlobalFocusSettings(), FocusFactor);
	}

	NewActiveSound.EnvelopeFollowerAttackTime = FMath::Max(EnvelopeFollowerAttackTime, 0);
	NewActiveSound.EnvelopeFollowerReleaseTime = FMath::Max(EnvelopeFollowerReleaseTime, 0);

	NewActiveSound.bUpdatePlayPercentage = OnAudioPlaybackPercentNative.IsBound() || OnAudioPlaybackPercent.IsBound();
	NewActiveSound.bUpdateSingleEnvelopeValue = OnAudioSingleEnvelopeValue.IsBound() || OnAudioSingleEnvelopeValueNative.IsBound();
	NewActiveSound.bUpdateMultiEnvelopeValue = OnAudioMultiEnvelopeValue.IsBound() || OnAudioMultiEnvelopeValueNative.IsBound();

	NewActiveSound.ModulationRouting = ModulationRouting;

	// Setup audio component cooked analysis data playback data set
	if (AudioDevice->IsBakedAnalaysisQueryingEnabled())
	{
		TArray<USoundWave*> SoundWavesWithCookedData;
		NewActiveSound.bUpdatePlaybackTime = Sound->GetSoundWavesWithCookedAnalysisData(SoundWavesWithCookedData);

		// Reset the audio component's soundwave playback times
		SoundWavePlaybackTimes.Reset();
		for (USoundWave* SoundWave : SoundWavesWithCookedData)
		{
			SoundWavePlaybackTimes.Add(SoundWave->GetUniqueID(), FSoundWavePlaybackTimeData(SoundWave));
		}
	}


	// Pass quantization data to the active sound
	NewActiveSound.QuantizedRequestData = InPlayRequestData.QuantizedRequestData;

	// Pass down any source buffer listener we have
	NewActiveSound.SetSourceListener(SourceBufferListener, bShouldSourceBufferListenerZeroBuffer);

	NewActiveSound.MaxDistance = MaxDistance;

	// Setup the submix and bus sends that may have been set before playing
	for (FSoundSubmixSendInfo& SubmixSendInfo : PendingSubmixSends)
	{
		NewActiveSound.SetSubmixSend(SubmixSendInfo);
	}
	PendingSubmixSends.Reset();

	for (FPendingSourceBusSendInfo& PendingBusSend : PendingBusSends)
	{
		NewActiveSound.SetSourceBusSend(PendingBusSend.BusSendType, PendingBusSend.BusSendInfo);
	}
	PendingBusSends.Reset();

	Audio::FVolumeFader& Fader = NewActiveSound.ComponentVolumeFader;
	Fader.SetVolume(0.0f); // Init to 0.0f to fade as default is 1.0f
	Fader.StartFade(InPlayRequestData.FadeVolumeLevel, InPlayRequestData.FadeInDuration, static_cast<Audio::EFaderCurve>(InPlayRequestData.FadeCurve));

	// Bump ActiveCount... this is used to determine if an audio component is still active after a sound reports back as completed
	++ActiveCount;

	// Pass along whether or not component is setup to support multiple active sounds.
	// This is to ensure virtualization will function accordingly. Disable the feature
	// if the sound is looping as a safety mechanism to avoid stuck loops.
	if (bIsSoundLooping)
	{
		if (bCanPlayMultipleInstances)
		{
			UE_LOGF(LogAudio, Warning, "'Can Play Multiple Instances' disabled: Sound '%ls' set to looping", *SoundToPlay->GetName());
		}

		AudioDevice->SetCanHaveMultipleActiveSounds(AudioComponentID, false);
	}
	else
	{
		AudioDevice->SetCanHaveMultipleActiveSounds(AudioComponentID, bCanPlayMultipleInstances);
	}

	TArray<FAudioParameter> SoundParams = DefaultParameters;
	
	if (AActor* Owner = GetOwner())
	{
		TArray<FAudioParameter> ActorParams;
		UActorSoundParameterInterface::Fill(Owner, ActorParams);
		FAudioParameter::Merge(MoveTemp(ActorParams), SoundParams);

		NewActiveSound.SetOwner(Owner);
	}

	TArray<FAudioParameter> InstanceParamsCopy = InstanceParameters;
	FAudioParameter::Merge(MoveTemp(InstanceParamsCopy), SoundParams);

#if UE_AUDIO_PROFILERTRACE_ENABLED
	const FString* EventLogID = &Audio::Trace::EventLog::ID::PlayRequestAudioComponent;
#else
	const FString* EventLogID = nullptr;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	AudioDevice->AddNewActiveSound(NewSharedActiveSound, MoveTemp(SoundParams), EventLogID);

	LastSoundPlayOrder = NewActiveSound.GetPlayOrder();

	// In editor, the audio thread is not run separate from the game thread, and can result in calling PlaybackComplete prior
	// to bIsActive being set. Therefore, we assign to the current state of ActiveCount as opposed to just setting to true.
	SetActiveFlag(ActiveCount > 0);

	BroadcastPlayState();
}

FAudioDevice* UAudioComponent::GetAudioDevice() const
{
	FAudioDevice* AudioDevice = nullptr;

	if (GEngine)
	{
		if (AudioDeviceID != INDEX_NONE)
		{
			FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
			AudioDevice = (AudioDeviceManager ? AudioDeviceManager->GetAudioDeviceRaw(AudioDeviceID) : nullptr);
		}
		else if (UWorld* World = GetWorld())
		{
			AudioDevice = World->GetAudioDeviceRaw();
		}
		else
		{
			AudioDevice = GEngine->GetMainAudioDeviceRaw();
		}
	}
	return AudioDevice;
}

FName UAudioComponent::GetFNameForStatID() const
{
	const USoundBase* SoundObject = Sound.Get();
	return SoundObject ? SoundObject->GetFNameForStatID() : Super::GetFNameForStatID();
}

void UAudioComponent::FadeIn(float FadeInDuration, float FadeVolumeLevel, float StartTime, const EAudioFaderCurve FadeCurve)
{
	PlayInternalRequestData Data;
	Data.StartTime = StartTime;
	Data.FadeInDuration = FadeInDuration;
	Data.FadeVolumeLevel = FadeVolumeLevel;
	Data.FadeCurve = FadeCurve;

	PlayInternal(Data);
}

void UAudioComponent::FadeOut(float FadeOutDuration, float FadeVolumeLevel, const EAudioFaderCurve FadeCurve)
{
	const bool bIsFadeOut = true;
	AdjustVolumeInternal(FadeOutDuration, FadeVolumeLevel, bIsFadeOut, FadeCurve);
}

void UAudioComponent::AdjustVolume(float AdjustVolumeDuration, float AdjustVolumeLevel, const EAudioFaderCurve FadeCurve)
{
	const bool bIsFadeOut = false;
	AdjustVolumeInternal(AdjustVolumeDuration, AdjustVolumeLevel, bIsFadeOut, FadeCurve);
}

void UAudioComponent::AdjustVolumeInternal(float AdjustVolumeDuration, float AdjustVolumeLevel, bool bInIsFadeOut, const EAudioFaderCurve FadeCurve)
{
	if (!IsActive())
	{
		return;
	}

	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return;
	}

	AdjustVolumeDuration = FMath::Max(0.0f, AdjustVolumeDuration);
	AdjustVolumeLevel = FMath::Max(0.0f, AdjustVolumeLevel);

	if (Sound && Sound->VirtualizationMode == EVirtualizationMode::Disabled && FMath::IsNearlyZero(AdjustVolumeDuration) && FMath::IsNearlyZero(AdjustVolumeLevel))
	{
		Stop();
		return;
	}

	const bool bWasFadingOut = bIsFadingOut;
	bIsFadingOut = bInIsFadeOut || FMath::IsNearlyZero(AdjustVolumeLevel);

	if (bWasFadingOut != bIsFadingOut)
	{
		BroadcastPlayState();
	}

	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AdjustVolume"), STAT_AudioAdjustVolume, STATGROUP_AudioThreadCommands);
	AudioDevice->SendCommandToActiveSounds(AudioComponentID, [bInIsFadeOut, AdjustVolumeLevel, AdjustVolumeDuration, FadeCurve](FActiveSound& ActiveSound)
	{
		Audio::FVolumeFader& Fader = ActiveSound.ComponentVolumeFader;
		const float InitialTargetVolume = Fader.GetTargetVolume();

		// Ignore fade out request if requested volume is higher than current target.
		if (bInIsFadeOut && AdjustVolumeLevel >= InitialTargetVolume)
		{
			return;
		}

		const bool ToZeroVolume = FMath::IsNearlyZero(AdjustVolumeLevel);
		if (ActiveSound.FadeOut == FActiveSound::EFadeOut::Concurrency)
		{
			// Ignore adjust volume request if non-zero and currently voice stealing.
			if (!FMath::IsNearlyZero(AdjustVolumeLevel))
			{
				return;
			}

			// Ignore request of longer fade out than active target if active is concurrency (voice stealing) fade.
			if (AdjustVolumeDuration > Fader.GetFadeDuration())
			{
				return;
			}
		}
		else
		{
			ActiveSound.FadeOut = bInIsFadeOut || ToZeroVolume ? FActiveSound::EFadeOut::User : FActiveSound::EFadeOut::None;
		}

		if (bInIsFadeOut || ToZeroVolume)
		{
			// If negative, active indefinitely, so always make sure set to minimum positive value for active fade.
			const float OldActiveDuration = Fader.GetActiveDuration();
			const float NewActiveDuration = OldActiveDuration < 0.0f
				? AdjustVolumeDuration
				: FMath::Min(OldActiveDuration, AdjustVolumeDuration);
			Fader.SetActiveDuration(NewActiveDuration);
		}

		Fader.StartFade(AdjustVolumeLevel, AdjustVolumeDuration, static_cast<Audio::EFaderCurve>(FadeCurve));
	}, GET_STATID(STAT_AudioAdjustVolume));
}

void UAudioComponent::ResetParameters()
{
	InstanceParameters.Reset();
	ISoundParameterControllerInterface::ResetParameters();
}

void UAudioComponent::Stop()
{
	if (!IsActive())
	{
		for (auto& Command : PendingQuartzCommandData)
		{
			Command.bHasBeenStoppedWhileQueued = true;
		}

		return;
	}

	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return;
	}

	if (bIsPreviewSound)
	{
		ResetParameters();
	}

	// Set this to immediately be inactive
	SetActiveFlag(false);

	UE_LOGF(LogAudio, Verbose, "%g: Stopping AudioComponent : '%ls' with Sound: '%ls'", GetAudioTimeSeconds(), *GetFullName(), Sound ? *Sound->GetName() : TEXT("nullptr"));

#if UE_AUDIO_PROFILERTRACE_ENABLED
	const FString* EventLogID = &Audio::Trace::EventLog::ID::StopRequestAudioComponent;
#else
	const FString* EventLogID = nullptr;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	AudioDevice->StopActiveSound(AudioComponentID, EventLogID);

	BroadcastPlayState();
}

void UAudioComponent::StopDelayed(float DelayTime)
{
	// 1. Stop immediately if no delay time
	if (DelayTime < 0.0f || FMath::IsNearlyZero(DelayTime))
	{
		Stop();
		return;
	}

	if (!IsActive())
	{
		return;
	}

	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return;
	}

	// 2. Performs delayed stop with no fade
	DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.StopDelayed"), STAT_AudioStopDelayed, STATGROUP_AudioThreadCommands);
	AudioDevice->SendCommandToActiveSounds(AudioComponentID, [DelayTime](FActiveSound& ActiveSound)
	{
		if (const USoundBase* StoppingSound = ActiveSound.GetSound())
		{
			UE_LOGF(LogAudio, Verbose, "%g: Delayed Stop requested for sound '%ls'",
				ActiveSound.GetWorld() ? ActiveSound.GetWorld()->GetAudioTimeSeconds() : 0.0f,
				*StoppingSound->GetName());
		}

		Audio::FVolumeFader& Fader = ActiveSound.ComponentVolumeFader;
		switch (ActiveSound.FadeOut)
		{
			case FActiveSound::EFadeOut::Concurrency:
				{
					// Ignore request of longer fade out than active target if active is concurrency (voice stealing) fade.
					if (DelayTime < Fader.GetFadeDuration())
					{
						Fader.SetActiveDuration(DelayTime);
					}
				}
				break;

			case FActiveSound::EFadeOut::User:
			case FActiveSound::EFadeOut::None:
			default:
			{
				ActiveSound.FadeOut = FActiveSound::EFadeOut::User;
				Fader.SetActiveDuration(DelayTime);
			}
			break;
		}
	}, GET_STATID(STAT_AudioStopDelayed));
}

void UAudioComponent::SetPaused(bool bPause)
{
	if (bIsPaused != bPause)
	{
		bIsPaused = bPause;

		if (IsActive())
		{
			UE_LOGF(LogAudio, Verbose, "%g: Pausing AudioComponent : '%ls' with Sound: '%ls'", GetAudioTimeSeconds(), *GetFullName(), Sound ? *Sound->GetName() : TEXT("nullptr"));

			if (FAudioDevice* AudioDevice = GetAudioDevice())
			{
				DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.PauseActiveSound"), STAT_AudioPauseActiveSound, STATGROUP_AudioThreadCommands);

				const uint64 MyAudioComponentID = AudioComponentID;
				FAudioThread::RunCommandOnAudioThread([AudioDevice, MyAudioComponentID, bPause]()
				{
					AudioDevice->PauseActiveSound(MyAudioComponentID, bPause);
				}, GET_STATID(STAT_AudioPauseActiveSound));
			}
		}

		BroadcastPlayState();
	}
}

void UAudioComponent::PlaybackCompleted(uint64 AudioComponentID, bool bFailedToStart)
{
	check(IsInAudioThread());

	DECLARE_CYCLE_STAT(TEXT("FGameThreadAudioTask.PlaybackCompleted"), STAT_AudioPlaybackCompleted, STATGROUP_TaskGraphTasks);

	FAudioThread::RunCommandOnGameThread([AudioComponentID, bFailedToStart]()
	{
		if (UAudioComponent* AudioComponent = GetAudioComponentFromID(AudioComponentID))
		{
			AudioComponent->PlaybackCompleted(bFailedToStart);
		}
	}, GET_STATID(STAT_AudioPlaybackCompleted));
}

void UAudioComponent::PlaybackCompleted(bool bFailedToStart)
{
	check(ActiveCount > 0);
	--ActiveCount;

	if (ActiveCount > 0)
	{
		return;
	}

	// Mark inactive before calling destroy to avoid recursion
	SetActiveFlag(false);

	if (!bFailedToStart && (OnAudioFinished.IsBound() || OnAudioFinishedNative.IsBound()))
	{
		INC_DWORD_STAT(STAT_AudioFinishedDelegatesCalled);
		SCOPE_CYCLE_COUNTER(STAT_AudioFinishedDelegates);

		OnAudioFinished.Broadcast();
		OnAudioFinishedNative.Broadcast(this);
	}

	// Auto destruction is handled via marking object for deletion.
	if (bAutoDestroy)
	{
		DestroyComponent();
	}
	// Otherwise see if we should detach ourself and wait until we're needed again
	else if (bAutoManageAttachment)
	{
		CancelAutoAttachment(true, GetWorld());
	}

	if (bIsPreviewSound)
	{
		ResetParameters();
	}

	BroadcastPlayState();
}

bool UAudioComponent::IsPlaying() const
{
	return IsActive();
}

bool UAudioComponent::IsVirtualized() const
{
	return bIsVirtualized;
}

EAudioComponentPlayState UAudioComponent::GetPlayState() const
{
	if (!IsActive())
	{
		return EAudioComponentPlayState::Stopped;
	}

	if (bIsPaused)
	{
		return EAudioComponentPlayState::Paused;
	}

	if (bIsFadingOut)
	{
		return EAudioComponentPlayState::FadingOut;
	}

	// Get the current audio time seconds and compare when it started and the fade in duration 
	float CurrentAudioTimeSeconds = GetAudioTimeSeconds();
	if (CurrentAudioTimeSeconds - TimeAudioComponentPlayed < FadeInTimeDuration)
	{
		return EAudioComponentPlayState::FadingIn;
	}

	// If we are not in any of the above states we are "playing"
	return EAudioComponentPlayState::Playing;
}

#if WITH_EDITORONLY_DATA
void UAudioComponent::UpdateSpriteTexture()
{
	if (SpriteComponent)
	{
		SpriteComponent->SpriteInfo.Category = TEXT("Sounds");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Sounds", "Sounds");

		FCookLoadScope EditorOnlyScope(ECookLoadType::EditorOnly);
		if (bAutoActivate)
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(nullptr, GAudioSpriteAssetNameAutoActivate));
		}
		else
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(nullptr, GAudioSpriteAssetName));
		}
	}
}
#endif

#if WITH_EDITOR
void UAudioComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (IsActive())
	{
		// If this is an auto destroy component we need to prevent it from being auto-destroyed since we're really just restarting it
		const bool bWasAutoDestroy = bAutoDestroy;
		bAutoDestroy = false;
		Stop();
		bAutoDestroy = bWasAutoDestroy;
		Play();
	}

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UAudioComponent, bCanPlayMultipleInstances))
	{
		GetAudioDevice()->SetCanHaveMultipleActiveSounds(AudioComponentID, bCanPlayMultipleInstances);
	}

#if WITH_EDITORONLY_DATA
	UpdateSpriteTexture();
#endif

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

const TObjectPtr<USoundAttenuation> UAudioComponent::GetAttenuationSettingsAsset() const
{
	if (AttenuationSettings)
	{
		return AttenuationSettings;
	}
	else if (Sound)
	{
		return Sound->AttenuationSettings;
	}
	return nullptr;
}

const FSoundAttenuationSettings* UAudioComponent::GetAttenuationSettingsToApply() const
{
	if (bOverrideAttenuation)
	{
		return &AttenuationOverrides;
	}
	else if (AttenuationSettings)
	{
		return &AttenuationSettings->Attenuation;
	}
	else if (Sound)
	{
		return Sound->GetAttenuationSettingsToApply();
	}
	return nullptr;
}

void UAudioComponent::SetAttenuationSettings(USoundAttenuation* InSoundAttenuation)
{
	AttenuationSettings = InSoundAttenuation;

	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		if (IsActive())
		{
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InSoundAttenuation](FActiveSound& ActiveSound)
			{
				ActiveSound.SetAttenuationSettingsAsset(InSoundAttenuation);
			});
		}
	}
}

void UAudioComponent::SetAttenuationOverrides(const FSoundAttenuationSettings& InAttenuationOverrides)
{
	AttenuationOverrides = InAttenuationOverrides;

	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		if (IsActive())
		{
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InAttenuationOverrides](FActiveSound& ActiveSound)
			{
				ActiveSound.AttenuationSettings = InAttenuationOverrides;
			});
		}
	}
}

void UAudioComponent::SetOverrideAttenuation(bool bInOverrideAttenuation)
{
	bOverrideAttenuation = bInOverrideAttenuation;

	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		if (IsActive())
		{
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [bInOverrideAttenuation](FActiveSound& ActiveSound)
			{
				ActiveSound.bIsAttenuationSettingsOverridden = bInOverrideAttenuation;
			});
		}
	}
}

bool UAudioComponent::BP_GetAttenuationSettingsToApply(FSoundAttenuationSettings& OutAttenuationSettings)
{
	if (const FSoundAttenuationSettings* Settings = GetAttenuationSettingsToApply())
	{
		OutAttenuationSettings = *Settings;
		return true;
	}
	return false;
}

void UAudioComponent::CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const
{
	const FSoundAttenuationSettings* AttenuationSettingsToApply = GetAttenuationSettingsToApply();

	if (AttenuationSettingsToApply)
	{
		AttenuationSettingsToApply->CollectAttenuationShapesForVisualization(ShapeDetailsMap);
	}

	// For sound cues we'll dig in and see if we can find any attenuation sound nodes that will affect the settings
	USoundCue* SoundCue = Cast<USoundCue>(Sound);
	if (SoundCue)
	{
		TArray<USoundNodeAttenuation*> AttenuationNodes;
		SoundCue->RecursiveFindAttenuation(SoundCue->FirstNode, AttenuationNodes);
		for (int32 NodeIndex = 0; NodeIndex < AttenuationNodes.Num(); ++NodeIndex)
		{
			AttenuationSettingsToApply = AttenuationNodes[NodeIndex]->GetAttenuationSettingsToApply();
			if (AttenuationSettingsToApply)
			{
				AttenuationSettingsToApply->CollectAttenuationShapesForVisualization(ShapeDetailsMap);
			}
		}
	}
}

void UAudioComponent::Activate(bool bReset)
{
	if (bReset || ShouldActivate() == true)
	{
		Play();
		if (IsActive())
		{
			OnComponentActivated.Broadcast(this, bReset);
		}
	}
}

void UAudioComponent::Deactivate()
{
	if (ShouldActivate() == false)
	{
		Stop();

		if (!IsActive())
		{
			OnComponentDeactivated.Broadcast(this);
		}
	}
}

void UAudioComponent::SetFadeInComplete()
{
	EAudioComponentPlayState PlayState = GetPlayState();
	if (PlayState != EAudioComponentPlayState::FadingIn)
	{
		BroadcastPlayState();
	}
}

void UAudioComponent::SetIsVirtualized(bool bInIsVirtualized)
{
	if (bIsVirtualized != bInIsVirtualized)
	{
		if (OnAudioVirtualizationChanged.IsBound())
		{
			OnAudioVirtualizationChanged.Broadcast(bInIsVirtualized);
		}

		if (OnAudioVirtualizationChangedNative.IsBound())
		{
			OnAudioVirtualizationChangedNative.Broadcast(this, bInIsVirtualized);
		}
	}

	bIsVirtualized = bInIsVirtualized ? 1 : 0;
}

void UAudioComponent::SetSourceBufferListener(const FSharedISourceBufferListenerPtr& InPtr, bool bShouldZeroBuffer)
{
	SourceBufferListener = InPtr;
	bShouldSourceBufferListenerZeroBuffer = bShouldZeroBuffer;
}

void UAudioComponent::SetWaveParameter(FName InName, USoundWave* InWave)
{
	SetObjectParameter(InName, Cast<UObject>(InWave));
}

void UAudioComponent::SetVolumeMultiplier(const float NewVolumeMultiplier)
{
	VolumeMultiplier = NewVolumeMultiplier;
	VolumeModulationMin = VolumeModulationMax = 1.f;

	if (IsActive())
	{
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetVolumeMultiplier"), STAT_AudioSetVolumeMultiplier, STATGROUP_AudioThreadCommands);
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [NewVolumeMultiplier](FActiveSound& ActiveSound)
			{
				ActiveSound.SetVolume(NewVolumeMultiplier);
			}, GET_STATID(STAT_AudioSetVolumeMultiplier));
		}
	}
}

void UAudioComponent::SetPitchMultiplier(const float NewPitchMultiplier)
{
	PitchMultiplier = NewPitchMultiplier;
	PitchModulationMin = PitchModulationMax = 1.f;

	if (IsActive())
	{
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetPitchMultiplier"), STAT_AudioSetPitchMultiplier, STATGROUP_AudioThreadCommands);
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [NewPitchMultiplier](FActiveSound& ActiveSound)
			{
				ActiveSound.SetPitch(NewPitchMultiplier);
			}, GET_STATID(STAT_AudioSetPitchMultiplier));
		}
	}
}

void UAudioComponent::SetUISound(const bool bInIsUISound)
{
	bIsUISound = bInIsUISound;

	if (IsActive())
	{
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetIsUISound"), STAT_AudioSetIsUISound, STATGROUP_AudioThreadCommands);
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [bInIsUISound](FActiveSound& ActiveSound)
			{
				ActiveSound.bIsUISound = bInIsUISound;
			}, GET_STATID(STAT_AudioSetIsUISound));
		}
	}
}

void UAudioComponent::AdjustAttenuation(const FSoundAttenuationSettings& InAttenuationSettings)
{
	bOverrideAttenuation = true;
	AttenuationOverrides = InAttenuationSettings;

	if (IsActive())
	{
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AdjustAttenuation"), STAT_AudioAdjustAttenuation, STATGROUP_AudioThreadCommands);
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InAttenuationSettings](FActiveSound& ActiveSound)
			{
				ActiveSound.AttenuationSettings = InAttenuationSettings;
			}, GET_STATID(STAT_AudioAdjustAttenuation));
		}
	}
}

void UAudioComponent::SetSubmixSend(USoundSubmixBase* Submix, float SendLevel)
{
	FSoundSubmixSendInfo SendInfo;
	SendInfo.SoundSubmix = Submix;
	SendInfo.SendLevel = SendLevel;

	if (IsPlaying())
	{
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AudioSetSubmixSend"), STAT_SetSubmixSend, STATGROUP_AudioThreadCommands);
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [SendInfo](FActiveSound& ActiveSound)
			{
				ActiveSound.SetSubmixSend(SendInfo);
			}, GET_STATID(STAT_SetSubmixSend));
		}
	}
	else
	{
		PendingSubmixSends.Add(SendInfo);
	}
}

void UAudioComponent::SetBusSendEffectInternal(USoundSourceBus* InSourceBus, UAudioBus* InAudioBus, float SendLevel, EBusSendType InBusSendType)
{
	FSoundSourceBusSendInfo BusSendInfo;
	BusSendInfo.SoundSourceBus = InSourceBus;
	BusSendInfo.AudioBus = InAudioBus;
	BusSendInfo.SendLevel = SendLevel;

	if (IsPlaying())
	{
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AudioSetBusSend"), STAT_SetBusSend, STATGROUP_AudioThreadCommands);
			AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InBusSendType, BusSendInfo](FActiveSound& ActiveSound)
			{
				// Preserve existing filter state - this function only changes send level,
				// so don't clobber filter enable/cutoff fields on an existing override entry.
				FSoundSourceBusSendInfo InfoToSend = BusSendInfo;
				TArray<FSoundSourceBusSendInfo> CurrentSends;
				ActiveSound.GetBusSends(InBusSendType, CurrentSends);
				for (const FSoundSourceBusSendInfo& Existing : CurrentSends)
				{
					const bool bSameSourceBus = InfoToSend.SoundSourceBus && (InfoToSend.SoundSourceBus == Existing.SoundSourceBus);
					const bool bSameAudioBus  = InfoToSend.AudioBus && (InfoToSend.AudioBus == Existing.AudioBus);
					if (bSameSourceBus || bSameAudioBus)
					{
						InfoToSend.bEnableLPFCutoff = Existing.bEnableLPFCutoff;
						InfoToSend.LPFCutoff = Existing.LPFCutoff;
						InfoToSend.bEnableHPFCutoff = Existing.bEnableHPFCutoff;
						InfoToSend.HPFCutoff = Existing.HPFCutoff;
						break;
					}
				}
				ActiveSound.SetSourceBusSend(InBusSendType, InfoToSend);
			}, GET_STATID(STAT_SetBusSend));
		}
	}
	else
	{
		FPendingSourceBusSendInfo NewPendingSourceBusInfo;
		NewPendingSourceBusInfo.BusSendInfo = BusSendInfo;
		NewPendingSourceBusInfo.BusSendType = InBusSendType;

		PendingBusSends.Add(NewPendingSourceBusInfo);
	}
}

void UAudioComponent::SetSourceBusSendPreEffect(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel)
{
	SetBusSendEffectInternal(SoundSourceBus, nullptr, SourceBusSendLevel, EBusSendType::PreEffect);
}

void UAudioComponent::SetSourceBusSendPostEffect(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel)
{
	SetBusSendEffectInternal(SoundSourceBus, nullptr, SourceBusSendLevel, EBusSendType::PostEffect);
}

void UAudioComponent::SetSourceBusSendPostAttenuation(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel)
{
	SetBusSendEffectInternal(SoundSourceBus, nullptr, SourceBusSendLevel, EBusSendType::PostAttenuation);
}

void UAudioComponent::SetAudioBusSendPreEffect(UAudioBus* AudioBus, float AudioBusSendLevel)
{
	SetBusSendEffectInternal(nullptr, AudioBus, AudioBusSendLevel, EBusSendType::PreEffect);
}

void UAudioComponent::SetAudioBusSendPostEffect(UAudioBus* AudioBus, float AudioBusSendLevel)
{
	SetBusSendEffectInternal(nullptr, AudioBus, AudioBusSendLevel, EBusSendType::PostEffect);
}

void UAudioComponent::SetAudioBusSendPostAttenuation(UAudioBus* AudioBus, float AudioBusSendLevel)
{
	SetBusSendEffectInternal(nullptr, AudioBus, AudioBusSendLevel, EBusSendType::PostAttenuation);
}

void UAudioComponent::SetLowPassFilterEnabled(bool InLowPassFilterEnabled)
{
	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetLowPassFilterFrequency"), STAT_AudioSetLowPassFilterEnabled, STATGROUP_AudioThreadCommands);
		AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InLowPassFilterEnabled](FActiveSound& ActiveSound)
		{
			ActiveSound.bEnableLowPassFilter = InLowPassFilterEnabled;
		}, GET_STATID(STAT_AudioSetLowPassFilterEnabled));
	}
}

void UAudioComponent::SetHighPassFilterEnabled(bool InHighPassFilterEnabled)
{
	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetHighPassFilterFrequency"), STAT_AudioSetHighPassFilterEnabled, STATGROUP_AudioThreadCommands);
		AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InHighPassFilterEnabled](FActiveSound& ActiveSound)
		{
			ActiveSound.bEnableHighPassFilter = InHighPassFilterEnabled;
		}, GET_STATID(STAT_AudioSetHighPassFilterEnabled));
	}
}

void UAudioComponent::SetLowPassFilterFrequency(float InLowPassFilterFrequency)
{
	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetLowPassFilterFrequency"), STAT_AudioSetLowPassFilterFrequency, STATGROUP_AudioThreadCommands);
		AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InLowPassFilterFrequency](FActiveSound& ActiveSound)
		{
			ActiveSound.LowPassFilterFrequency = InLowPassFilterFrequency;
		}, GET_STATID(STAT_AudioSetLowPassFilterFrequency));
	}
}

void UAudioComponent::SetHighPassFilterFrequency(float InHighPassFilterFrequency)
{
	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetHighPassFilterFrequency"), STAT_AudioSetHighPassFilterFrequency, STATGROUP_AudioThreadCommands);
		AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InHighPassFilterFrequency](FActiveSound& ActiveSound)
		{
			ActiveSound.HighPassFilterFrequency = InHighPassFilterFrequency;
		}, GET_STATID(STAT_AudioSetHighPassFilterFrequency));
	}
}

void UAudioComponent::SetOutputToBusOnly(bool bInOutputToBusOnly)
{
	if (FAudioDevice* AudioDevice = GetAudioDevice())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetOutputToBusOnly"), STAT_AudioSetOutputToBusOnly, STATGROUP_AudioThreadCommands);
		AudioDevice->SendCommandToActiveSounds(AudioComponentID, [bInOutputToBusOnly](FActiveSound& ActiveSound)
		{
			ActiveSound.bHasActiveMainSubmixOutputOverride = true;
			ActiveSound.bHasActiveSubmixSendRoutingOverride = true;
			if (bInOutputToBusOnly)
			{
				ActiveSound.bHasActiveBusSendRoutingOverride = true;
				ActiveSound.bEnableBusSendRoutingOverride = true;
			}
			ActiveSound.bEnableMainSubmixOutputOverride = !bInOutputToBusOnly;
			ActiveSound.bEnableSubmixSendRoutingOverride = !bInOutputToBusOnly;
		});
	}
}

bool UAudioComponent::HasCookedFFTData() const
{
	if (Sound)
	{
		return Sound->HasCookedFFTData();
	}
	return false;
}

bool UAudioComponent::HasCookedAmplitudeEnvelopeData() const
{
	if (Sound)
	{
		return Sound->HasCookedAmplitudeEnvelopeData();
	}
	return false;
}

void UAudioComponent::SetPlaybackTimes(const TMap<uint32, float>& InSoundWavePlaybackTimes)
{
	// Reset the playback times for everything in case the wave instance stops and is not updated
	for (auto& Elem : SoundWavePlaybackTimes)
	{
		Elem.Value.PlaybackTime = 0.0f;
	}

	for (auto& Elem : InSoundWavePlaybackTimes)
	{
		uint32 ObjectId = Elem.Key;
		FSoundWavePlaybackTimeData* PlaybackTimeData = SoundWavePlaybackTimes.Find(ObjectId);
		if (PlaybackTimeData)
		{
			PlaybackTimeData->PlaybackTime = FMath::Max(Elem.Value - BakedAnalysisTimeShiftCVar, 0.0f);
		}
	}
}

bool UAudioComponent::GetCookedFFTData(const TArray<float>& FrequenciesToGet, TArray<FSoundWaveSpectralData>& OutSoundWaveSpectralData)
{
	bool bHadData = false;
	if (IsPlaying() && SoundWavePlaybackTimes.Num() > 0 && FrequenciesToGet.Num() > 0)
	{
		OutSoundWaveSpectralData.Reset();
		for (float Frequency : FrequenciesToGet)
		{
			FSoundWaveSpectralData NewEntry;
			NewEntry.FrequencyHz = Frequency;
			OutSoundWaveSpectralData.Add(NewEntry);
		}

		// Sort by frequency (lowest frequency first).
		OutSoundWaveSpectralData.Sort(FCompareSpectralDataByFrequencyHz());

		int32 NumEntriesAdded = 0;
		for (auto& Entry : SoundWavePlaybackTimes)
		{
			if (Entry.Value.PlaybackTime > 0.0f && Entry.Value.SoundWave->CookedSpectralTimeData.Num() > 0)
			{
				static TArray<FSoundWaveSpectralData> CookedSpectralData;
				CookedSpectralData.Reset();

				// Find the point in the spectral data that corresponds to the time
				Entry.Value.SoundWave->GetInterpolatedCookedFFTDataForTime(Entry.Value.PlaybackTime, Entry.Value.LastFFTCookedIndex, CookedSpectralData, Sound->IsLooping());

				if (CookedSpectralData.Num() > 0)
				{
					// Find the interpolated values given the frequencies we want to get
					for (FSoundWaveSpectralData& OutSpectralData : OutSoundWaveSpectralData)
					{
						// Check min edge case: we're requesting cooked FFT data lower than what we have cooked
						if (OutSpectralData.FrequencyHz < CookedSpectralData[0].FrequencyHz)
						{
							// Just mix in the lowest value we have cooked
							OutSpectralData.Magnitude += CookedSpectralData[0].Magnitude;
							OutSpectralData.NormalizedMagnitude += CookedSpectralData[0].NormalizedMagnitude;
						}
						// Check max edge case: we're requesting cooked FFT data at a higher frequency than what we have cooked
						else if (OutSpectralData.FrequencyHz >= CookedSpectralData.Last().FrequencyHz)
						{
							// Just mix in the highest value we have cooked
							OutSpectralData.Magnitude += CookedSpectralData.Last().Magnitude;
							OutSpectralData.NormalizedMagnitude += CookedSpectralData.Last().NormalizedMagnitude;
						}
						// We need to find the 2 closest cooked results and interpolate those
						else
						{
							for (int32 SpectralDataIndex = 0; SpectralDataIndex < CookedSpectralData.Num() - 1; ++SpectralDataIndex)
							{
								const FSoundWaveSpectralData& CurrentSpectralData = CookedSpectralData[SpectralDataIndex];
								const FSoundWaveSpectralData& NextSpectralData = CookedSpectralData[SpectralDataIndex + 1];
								if (OutSpectralData.FrequencyHz >= CurrentSpectralData.FrequencyHz && OutSpectralData.FrequencyHz < NextSpectralData.FrequencyHz)
								{
									float Alpha = (OutSpectralData.FrequencyHz - CurrentSpectralData.FrequencyHz) / (NextSpectralData.FrequencyHz - CurrentSpectralData.FrequencyHz);
									OutSpectralData.Magnitude += FMath::Lerp(CurrentSpectralData.Magnitude, NextSpectralData.Magnitude, Alpha);
									OutSpectralData.NormalizedMagnitude += FMath::Lerp(CurrentSpectralData.NormalizedMagnitude, NextSpectralData.NormalizedMagnitude, Alpha);

									break;
								}
							}
						}
					}

					++NumEntriesAdded;
					bHadData = true;
				}
			}
		}

		// Divide by the number of entries we added (i.e. we are averaging together multiple cooked FFT data in the case of multiple sound waves playing with cooked data)
		if (NumEntriesAdded > 1)
		{
			for (FSoundWaveSpectralData& OutSpectralData : OutSoundWaveSpectralData)
			{
				OutSpectralData.Magnitude /= NumEntriesAdded;
				OutSpectralData.NormalizedMagnitude /= NumEntriesAdded;
			}
		}
	}

	return bHadData;
}

bool UAudioComponent::GetCookedFFTDataForAllPlayingSounds(TArray<FSoundWaveSpectralDataPerSound>& OutSoundWaveSpectralData)
{
	bool bHadData = false;
	if (IsPlaying() && SoundWavePlaybackTimes.Num() > 0)
	{
		OutSoundWaveSpectralData.Reset();

		for (auto& Entry : SoundWavePlaybackTimes)
		{
			if (Entry.Value.PlaybackTime > 0.0f && Entry.Value.SoundWave->CookedSpectralTimeData.Num() > 0)
			{
				FSoundWaveSpectralDataPerSound NewOutput;
				NewOutput.SoundWave = Entry.Value.SoundWave;
				NewOutput.PlaybackTime = Entry.Value.PlaybackTime;

				// Find the point in the spectral data that corresponds to the time
				Entry.Value.SoundWave->GetInterpolatedCookedFFTDataForTime(Entry.Value.PlaybackTime, Entry.Value.LastFFTCookedIndex, NewOutput.SpectralData, Sound->IsLooping());
				if (NewOutput.SpectralData.Num())
				{
					OutSoundWaveSpectralData.Add(NewOutput);
					bHadData = true;
				}
			}
		}
	}
	return bHadData;
}

bool UAudioComponent::GetCookedEnvelopeData(float& OutEnvelopeData)
{
	bool bHadData = false;
	if (IsPlaying() && SoundWavePlaybackTimes.Num() > 0)
	{
		static TArray<FSoundWaveEnvelopeTimeData> CookedEnvelopeData;
		int32 NumEntriesAdded = 0;
		OutEnvelopeData = 0.0f;
		for (auto& Entry : SoundWavePlaybackTimes)
		{
			if (Entry.Value.SoundWave->CookedEnvelopeTimeData.Num() > 0 && Entry.Value.PlaybackTime > 0.0f)
			{
				CookedEnvelopeData.Reset();

				// Find the point in the spectral data that corresponds to the time
				float SoundWaveAmplitude = 0.0f;
				if (Entry.Value.SoundWave->GetInterpolatedCookedEnvelopeDataForTime(Entry.Value.PlaybackTime, Entry.Value.LastEnvelopeCookedIndex, SoundWaveAmplitude, Sound->IsLooping()))
				{
					OutEnvelopeData += SoundWaveAmplitude;
					++NumEntriesAdded;
					bHadData = true;
				}
			}
		}

		// Divide by number of entries we added... get average amplitude envelope
		if (bHadData)
		{
			OutEnvelopeData /= NumEntriesAdded;
		}
	}

	return bHadData;
}

bool UAudioComponent::GetCookedEnvelopeDataForAllPlayingSounds(TArray<FSoundWaveEnvelopeDataPerSound>& OutEnvelopeData)
{
	bool bHadData = false;
	if (IsPlaying() && SoundWavePlaybackTimes.Num() > 0)
	{
		for (auto& Entry : SoundWavePlaybackTimes)
		{
			if (Entry.Value.SoundWave->CookedEnvelopeTimeData.Num() > 0 && Entry.Value.PlaybackTime > 0.0f)
			{
				// Find the point in the spectral data that corresponds to the time
				float SoundWaveAmplitude = 0.0f;
				if (Entry.Value.SoundWave->GetInterpolatedCookedEnvelopeDataForTime(Entry.Value.PlaybackTime, Entry.Value.LastEnvelopeCookedIndex, SoundWaveAmplitude, Sound->IsLooping()))
				{
					FSoundWaveEnvelopeDataPerSound NewOutput;
					NewOutput.SoundWave = Entry.Value.SoundWave;
					NewOutput.PlaybackTime = Entry.Value.PlaybackTime;
					NewOutput.Envelope = SoundWaveAmplitude;
					OutEnvelopeData.Add(NewOutput);
					bHadData = true;
				}

			}
		}
	}
	return bHadData;
}

void UAudioComponent::SetModulationRouting(const TSet<USoundModulatorBase*>& Modulators, const EModulationDestination Destination, const EModulationRouting RoutingMethod)
{
	struct FModulatorWeakRoutingAccessor
	{
	private:
		TArray<TWeakObjectPtr<UObject>> WeakCollection;
		const FSoundModulationDefaultRoutingSettings Settings;

	public:
		FModulatorWeakRoutingAccessor(const FSoundModulationDefaultRoutingSettings& RoutingSettings)
			: Settings(RoutingSettings)
		{
			auto GatherModulators = [&](const TSet<TObjectPtr<USoundModulatorBase>>& InModulators)
			{
				for (const TObjectPtr<USoundModulatorBase>& Modulator : InModulators)
				{
					if (USoundModulatorBase* ModPtr = Modulator.Get())
					{
						WeakCollection.Add(TWeakObjectPtr<UObject>(ModPtr));
					}
				}
			};

			GatherModulators(RoutingSettings.VolumeModulationDestination.Modulators);
			GatherModulators(RoutingSettings.PitchModulationDestination.Modulators);
			GatherModulators(RoutingSettings.HighpassModulationDestination.Modulators);
			GatherModulators(RoutingSettings.LowpassModulationDestination.Modulators);
		}

		const FSoundModulationDefaultRoutingSettings* GetSafe() const
		{
			auto IsValid = [](const TWeakObjectPtr<UObject>& WeakPtr) { return WeakPtr.IsValid(); };
			if (Algo::AllOf(WeakCollection, IsValid))
			{
				return &Settings;
			}

			return nullptr;
		}
	};

	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return;
	}

	switch (Destination)
	{
		case EModulationDestination::Volume:
			ModulationRouting.VolumeRouting = RoutingMethod;
			ModulationRouting.VolumeModulationDestination.Modulators = Modulators;
			break;
		case EModulationDestination::Pitch:
			ModulationRouting.PitchRouting = RoutingMethod;
			ModulationRouting.PitchModulationDestination.Modulators = Modulators;
			break;
		case EModulationDestination::Lowpass:
			ModulationRouting.LowpassRouting = RoutingMethod;
			ModulationRouting.LowpassModulationDestination.Modulators = Modulators;
			break;
		case EModulationDestination::Highpass:
			ModulationRouting.HighpassRouting = RoutingMethod;
			ModulationRouting.HighpassModulationDestination.Modulators = Modulators;
			break;
		default:
		{
			static_assert(static_cast<int32>(EModulationDestination::Count) == 4, "Possible missing ELiteralType case coverage.");
			ensureMsgf(false, TEXT("Failed to set input node default: Literal type not supported"));
			return;
		}
	}

	// Tell the active sounds on the component to use the new Modulation Routing
	AudioDevice->SendCommandToActiveSounds(AudioComponentID, [NewRouting = FModulatorWeakRoutingAccessor(ModulationRouting)](FActiveSound& ActiveSound)
	{
		if (const FSoundModulationDefaultRoutingSettings* SafeSettings = NewRouting.GetSafe())
		{
			ActiveSound.SetNewModulationRouting(*SafeSettings);
		}
	});

}

void UAudioComponent::AddModulationRouting(const TSet<USoundModulatorBase*>& Modulators, const EModulationDestination Destination)
{
	using namespace AudioComponentPrivate;

	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return;
	}

	TSet<TObjectPtr<USoundModulatorBase>> ConvertedModulators;
	for (USoundModulatorBase* Modulator : Modulators)
	{
		ConvertedModulators.Add(TObjectPtr<USoundModulatorBase>(Modulator));
	}

	switch (Destination)
	{
		case EModulationDestination::Volume:
			ModulationRouting.VolumeRouting = EModulationRouting::Union;
			ModulationRouting.VolumeModulationDestination.Modulators.Append(ConvertedModulators);
			break;
		case EModulationDestination::Pitch:
			ModulationRouting.PitchRouting = EModulationRouting::Union;
			ModulationRouting.PitchModulationDestination.Modulators.Append(ConvertedModulators);
			break;
		case EModulationDestination::Lowpass:
			ModulationRouting.LowpassRouting = EModulationRouting::Union;
			ModulationRouting.LowpassModulationDestination.Modulators.Append(ConvertedModulators);
			break;
		case EModulationDestination::Highpass:
			ModulationRouting.HighpassRouting = EModulationRouting::Union;
			ModulationRouting.HighpassModulationDestination.Modulators.Append(ConvertedModulators);
			break;
		default:
		{
			static_assert(static_cast<int32>(EModulationDestination::Count) == 4, "Possible missing ELiteralType case coverage.");
			ensureMsgf(false, TEXT("Failed to set input node default: Literal type not supported"));
			return;
		}
	}

	// Tell the active sounds on the component to use the new Modulation Routing
	AudioDevice->SendCommandToActiveSounds(AudioComponentID, [Accessor = FModulatorWeakAccessor(ConvertedModulators), Destination](FActiveSound& ActiveSound)
	{
		TSet<TObjectPtr<USoundModulatorBase>> NewModulators;
		if (Accessor.TryGetSafe(NewModulators))
		{
			ActiveSound.AddModulationRouting(NewModulators, Destination);
		}
	});
}

void UAudioComponent::RemoveModulationRouting(const TSet<USoundModulatorBase*>& Modulators, const EModulationDestination Destination)
{
	using namespace AudioComponentPrivate;

	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return;
	}

	TSet<TObjectPtr<USoundModulatorBase>> ConvertedModulators;
	for (USoundModulatorBase* Modulator : Modulators)
	{
		ConvertedModulators.Add(TObjectPtr<USoundModulatorBase>(Modulator));
	}

	switch (Destination)
	{
		case EModulationDestination::Volume:
			ModulationRouting.VolumeModulationDestination.Modulators = ModulationRouting.VolumeModulationDestination.Modulators.Difference(ConvertedModulators);
			break;
		case EModulationDestination::Pitch:
			ModulationRouting.PitchModulationDestination.Modulators = ModulationRouting.PitchModulationDestination.Modulators.Difference(ConvertedModulators);
			break;
		case EModulationDestination::Lowpass:
			ModulationRouting.LowpassModulationDestination.Modulators = ModulationRouting.LowpassModulationDestination.Modulators.Difference(ConvertedModulators);
			break;
		case EModulationDestination::Highpass:
			ModulationRouting.HighpassModulationDestination.Modulators = ModulationRouting.HighpassModulationDestination.Modulators.Difference(ConvertedModulators);
			break;
		default:
		{
			static_assert(static_cast<int32>(EModulationDestination::Count) == 4, "Possible missing ELiteralType case coverage.");
			ensureMsgf(false, TEXT("Failed to set input node default: Literal type not supported"));
			return;
		}
	}

	// Tell the active sounds on the component to remove the given modulators
	AudioDevice->SendCommandToActiveSounds(AudioComponentID, [Accessor = FModulatorWeakAccessor(ConvertedModulators), Destination](FActiveSound& ActiveSound)
	{
		TSet<TObjectPtr<USoundModulatorBase>> ModulatorsToRemove;
		if (Accessor.TryGetSafe(ModulatorsToRemove))
		{
			ActiveSound.RemoveModulationRouting(ModulatorsToRemove, Destination);
		}
	});
}

TSet<USoundModulatorBase*> UAudioComponent::GetModulators(const EModulationDestination Destination)
{
	FAudioDevice* AudioDevice = GetAudioDevice();
	if (!AudioDevice)
	{
		return TSet<USoundModulatorBase*>();
	}
	
	const TSet<TObjectPtr<USoundModulatorBase>>* ModulatorSet = nullptr;

	switch (Destination)
	{
		case EModulationDestination::Volume:
			ModulatorSet = &ModulationRouting.VolumeModulationDestination.Modulators;
			break;
		case EModulationDestination::Pitch:
			ModulatorSet = &ModulationRouting.PitchModulationDestination.Modulators;
			break;
		case EModulationDestination::Lowpass:
			ModulatorSet = &ModulationRouting.LowpassModulationDestination.Modulators;
			break;
		case EModulationDestination::Highpass:
			ModulatorSet = &ModulationRouting.HighpassModulationDestination.Modulators;
			break;
		default:
		{
			static_assert(static_cast<int32>(EModulationDestination::Count) == 4, "Possible missing ELiteralType case coverage.");
			ensureMsgf(false, TEXT("Failed to set input node default: Literal type not supported"));
			return TSet<USoundModulatorBase*>();
		}
	}

	check(ModulatorSet);

	TSet<USoundModulatorBase*> Modulators;
	for (const TObjectPtr<USoundModulatorBase>& Modulator : *ModulatorSet)
	{
		Modulators.Add(Modulator.Get());
	}

	return Modulators;
}

void UAudioComponent::SetSourceEffectChain(USoundEffectSourcePresetChain* InSourceEffectChain)
{
	SourceEffectChain = InSourceEffectChain;
}

