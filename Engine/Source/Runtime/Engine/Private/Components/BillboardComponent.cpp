// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/BillboardComponent.h"
#include "MeshElementCollector.h"
#include "SceneView.h"
#include "UObject/ConstructorHelpers.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Components/LightComponent.h"
#include "Engine/CollisionProfile.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Texture2D.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/Light.h"
#include "Engine/Engine.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BillboardComponent)

namespace BillboardConstants
{
	static const float DefaultScreenSize = 0.0025f;
}

#if WITH_EDITORONLY_DATA
float UBillboardComponent::EditorScale = 1.0f;
#endif

/** Represents a billboard sprite to the scene manager. */
class FSpriteSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	/** Initialization constructor. */
	FSpriteSceneProxy(const UBillboardComponent* InComponent, float SpriteScale)
		: FPrimitiveSceneProxy(InComponent)
		, ScreenSize(InComponent->ScreenSize)
		, U(InComponent->U)
		, V(InComponent->V)
		, Color(FLinearColor::White)
		, bIsScreenSizeScaled(InComponent->bIsScreenSizeScaled)
#if WITH_EDITORONLY_DATA
		, bUseInEditorScaling(InComponent->bUseInEditorScaling)
		, EditorScale(InComponent->EditorScale)
#endif
	{
#if WITH_EDITOR
		// Extract the sprite category from the component if in the editor
		if ( GIsEditor )
		{
			SpriteCategoryIndex = GEngine->GetSpriteCategoryIndex( InComponent->SpriteInfo.Category );
		}
#endif //WITH_EDITOR

		bWillEverBeLit = false;

		// Calculate the scale factor for the sprite.
		Scale = InComponent->GetComponentTransform().GetMaximumAxisScale() * SpriteScale * 0.25f;

		OpacityMaskRefVal = InComponent->OpacityMaskRefVal;

		if(InComponent->Sprite)
		{
			Texture = InComponent->Sprite;

			// Set UL and VL to the size of the texture if they are set to 0.0, otherwise use the given value
			ComponentUL = InComponent->UL;
			ComponentVL = InComponent->VL;
		}
		else
		{
			Texture = NULL;
			ComponentUL = ComponentVL = 0;
		}

		if (AActor* Owner = InComponent->GetOwner())
		{
			// If the owner of this sprite component is an ALight, tint the sprite
			// to match the lights color.  
			if (ALight* Light = Cast<ALight>(Owner))
			{
				if (Light->GetLightComponent())
				{
					Color = Light->GetLightComponent()->LightColor.ReinterpretAsLinear();
					Color.A = 255;
				}
			}
#if WITH_EDITORONLY_DATA
			else if (GIsEditor)
			{
				Color = FLinearColor::White;
			}
#endif // WITH_EDITORONLY_DATA

			//save off override states
#if WITH_EDITORONLY_DATA
			bIsActorLocked = InComponent->bShowLockedLocation && Owner->IsLockLocation();
#else // WITH_EDITORONLY_DATA
			bIsActorLocked = false;
#endif // WITH_EDITORONLY_DATA
		}
	}

	// FPrimitiveSceneProxy interface.
	
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER( STAT_SpriteSceneProxy_GetDynamicMeshElements );

		const FTexture* TextureResource = Texture ? Texture->GetResource() : nullptr;
		if (TextureResource)
		{
			const float UL = ComponentUL == 0.0f ? TextureResource->GetSizeX() : ComponentUL;
			const float VL = ComponentVL == 0.0f ? TextureResource->GetSizeY() : ComponentVL;

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];

					// Calculate the view-dependent scaling factor.
					float ViewedSizeX = Scale * UL;
					float ViewedSizeY = Scale * VL;

					if (bIsScreenSizeScaled && (View->ViewMatrices.GetViewToClip().M[3][3] != 1.0f))
					{
						const float ZoomFactor	= FMath::Min<float>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);

						if(ZoomFactor != 0.0f)
						{
							const float Radius = View->WorldToScreen(Origin).W * (ScreenSize / ZoomFactor);

							if (Radius < 1.0f)
							{
								ViewedSizeX *= Radius;
								ViewedSizeY *= Radius;
							}					
						}
					}

#if WITH_EDITORONLY_DATA
					ViewedSizeX *= EditorScale;
					ViewedSizeY *= EditorScale;
#endif

					FLinearColor ColorToUse = Color;

					// Set the selection/hover color from the current engine setting.
					// The color is multiplied by 10 because this value is normally expected to be blended
					// additively, this is not how the sprites work and therefore need an extra boost
					// to appear the same color as previously
#if WITH_EDITOR
					if( View->bHasSelectedComponents && !IsIndividuallySelected() )
					{
						ColorToUse = FLinearColor::White + (GEngine->GetSubduedSelectionOutlineColor() * GEngine->SelectionHighlightIntensityBillboards * 10);
					}
					else 
#endif
					if (IsSelected())
					{
						ColorToUse = FLinearColor::White + (GEngine->GetSelectedMaterialColor() * GEngine->SelectionHighlightIntensityBillboards * 10);
					}
					else if (IsHovered())
					{
						ColorToUse = FLinearColor::White + (GEngine->GetHoveredMaterialColor() * GEngine->SelectionHighlightIntensityBillboards * 10);
					}

					// Sprites of locked actors draw in red.
					if (bIsActorLocked)
					{
						ColorToUse = FColor::Red;
					}
					FLinearColor PrimitiveColorToUse = IsSelected() ? ColorToUse : (FLinearColor)GetPrimitiveColor();

					ColorToUse.A = 1.0f;

					const FLinearColor& SpriteColor = View->Family->EngineShowFlags.ActorColoration ? PrimitiveColorToUse : ColorToUse;

					Collector.GetPDI(ViewIndex)->DrawSprite(
						Origin,
						ViewedSizeX,
						ViewedSizeY,
						TextureResource,
						SpriteColor,
						GetDepthPriorityGroup(View),
						U,UL,V,VL,
						SE_BLEND_Masked,
						OpacityMaskRefVal
						);
				}
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					RenderBounds(Collector.GetPDI(ViewIndex), View->Family->EngineShowFlags, GetBounds(), IsSelected());
				}
			}
#endif
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		bool bVisible = View->Family->EngineShowFlags.BillboardSprites;
#if WITH_EDITOR
		if ( GIsEditor && bVisible && SpriteCategoryIndex != INDEX_NONE && SpriteCategoryIndex < View->SpriteCategoryVisibility.Num() )
		{
			bVisible = View->SpriteCategoryVisibility[ SpriteCategoryIndex ];
		}
#endif
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View) && bVisible;
		Result.bOpaque = true;
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		return Result;
	}
	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override
	{
		Origin = GetLocalToWorld().GetOrigin();
	}
	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }
	
	uint32 GetAllocatedSize(void) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

private:
	FVector Origin;
	const float ScreenSize;
	const UTexture2D* Texture;
	float Scale;
	const float U;
	float ComponentUL;
	const float V;
	float ComponentVL;
	float OpacityMaskRefVal;
	FLinearColor Color;
	const uint32 bIsScreenSizeScaled : 1;
	uint32 bIsActorLocked : 1;
#if WITH_EDITORONLY_DATA
	int32 SpriteCategoryIndex;
	bool bUseInEditorScaling;
	float EditorScale;
#endif // #if WITH_EDITORONLY_DATA
};

UBillboardComponent::UBillboardComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> SpriteTexture;
		FName ID_Misc;
		FText NAME_Misc;
		FConstructorStatics()
			: SpriteTexture(TEXT("/Engine/EditorResources/S_Actor"))
			, ID_Misc(TEXT("Misc"))
			, NAME_Misc(NSLOCTEXT( "SpriteCategory", "Misc", "Misc" ))
		{
		}
	};

	static FConstructorStatics ConstructorStatics;
	Sprite = ConstructorStatics.SpriteTexture.Object;
#endif

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SetUsingAbsoluteScale(true);

	bIsScreenSizeScaled = false;
	ScreenSize = BillboardConstants::DefaultScreenSize;
	U = 0;
	V = 0;
	UL = 0;
	VL = 0;
	OpacityMaskRefVal = .5f;
	bHiddenInGame = true;
	SetGenerateOverlapEvents(false);
	bUseEditorCompositing = true;
	bExcludeFromLightAttachmentGroup = true;

#if WITH_EDITORONLY_DATA
	SpriteInfo.Category = ConstructorStatics.ID_Misc;
	SpriteInfo.DisplayName = ConstructorStatics.NAME_Misc;
	bUseInEditorScaling = true;
#endif // WITH_EDITORONLY_DATA
}

FPrimitiveSceneProxy* UBillboardComponent::CreateSceneProxy()
{
	float SpriteScale = 1.0f;
#if WITH_EDITOR
	if (GetOwner())
	{
		SpriteScale = GetOwner()->SpriteScale;
	}
#endif
	return new FSpriteSceneProxy(this, SpriteScale);
}

FBoxSphereBounds UBillboardComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const float NewScale = LocalToWorld.GetScale3D().GetMax() * (Sprite ? (float)FMath::Max(Sprite->GetSizeX(),Sprite->GetSizeY()) : 1.0f);
	return FBoxSphereBounds(LocalToWorld.GetLocation(),FVector(NewScale,NewScale,NewScale),FMath::Sqrt(3.0f * FMath::Square(NewScale)));
}

bool UBillboardComponent::IsShown(const FEngineShowFlags& ShowFlags) const
{
	return ShowFlags.BillboardSprites;
}

#if WITH_EDITOR
bool UBillboardComponent::ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	AActor* Actor = GetOwner();

	if (!bConsiderOnlyBSP && Sprite != nullptr && Actor != nullptr)
	{
		const float Scale = GetComponentTransform().GetMaximumAxisScale();

		// Construct a box representing the sprite
		const FBox SpriteBox(
			Actor->GetActorLocation() - Scale * FMath::Max(Sprite->GetSizeX(), Sprite->GetSizeY()) * FVector(0.5f, 0.5f, 0.5f),
			Actor->GetActorLocation() + Scale * FMath::Max(Sprite->GetSizeX(), Sprite->GetSizeY()) * FVector(0.5f, 0.5f, 0.5f));

		// If the selection box doesn't have to encompass the entire component and it intersects with the box constructed for the sprite, then it is valid.
		// Additionally, if the selection box does have to encompass the entire component and both the min and max vectors of the sprite box are inside the selection box,
		// then it is valid.
		if ((!bMustEncompassEntireComponent && InSelBBox.Intersect(SpriteBox))
			|| (bMustEncompassEntireComponent && InSelBBox.IsInside(SpriteBox)))
		{
			return true;
		}
	}

	return false;
}

bool UBillboardComponent::ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	AActor* Actor = GetOwner();

	if (!bConsiderOnlyBSP && Sprite != nullptr && Actor != nullptr)
	{
		const float Scale = GetComponentTransform().GetMaximumAxisScale();
		const float MaxExtent = FMath::Max(Sprite->GetSizeX(), Sprite->GetSizeY());
		const FVector Extent = Scale * MaxExtent * FVector(0.5f, 0.5f, 0.0f);

		bool bIsFullyContained;
		if (InFrustum.IntersectBox(Actor->GetActorLocation(), Extent, bIsFullyContained))
		{
			return !bMustEncompassEntireComponent || bIsFullyContained;
		}
	}

	return false;
}
#endif

void UBillboardComponent::SetSprite(UTexture2D* NewSprite)
{
	Sprite = NewSprite;
	MarkRenderStateDirty();
}

void UBillboardComponent::SetUV(int32 NewU, int32 NewUL, int32 NewV, int32 NewVL)
{
	U = NewU;
	UL = NewUL;
	V = NewV;
	VL = NewVL;
	MarkRenderStateDirty();
}

void UBillboardComponent::SetSpriteAndUV(UTexture2D* NewSprite, int32 NewU, int32 NewUL, int32 NewV, int32 NewVL)
{
	U = NewU;
	UL = NewUL;
	V = NewV;
	VL = NewVL;
	SetSprite(NewSprite);
}

void UBillboardComponent::SetOpacityMaskRefVal(float RefVal)
{
	OpacityMaskRefVal = RefVal;
	MarkRenderStateDirty();
}

#if WITH_EDITORONLY_DATA
void UBillboardComponent::SetEditorScale(float InEditorScale)
{
	EditorScale = InEditorScale;
	for(TObjectIterator<UBillboardComponent> It; It; ++It)
	{
		It->MarkRenderStateDirty();
	}
}
#endif

