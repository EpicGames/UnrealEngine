// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/BoundsCopyComponent.h"

#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoundsCopyComponent)

UBoundsCopyComponent::UBoundsCopyComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsEditorOnly = true;
}

#if WITH_EDITOR

void UBoundsCopyComponent::SetRotation()
{
	if (BoundsSourceActor.IsValid())
	{
		// Copy the source actor rotation and notify the parent actor
		GetOwner()->Modify();
		GetOwner()->SetActorRotation(BoundsSourceActor->GetTransform().GetRotation());
		GetOwner()->PostEditMove(true);
	}
}

void UBoundsCopyComponent::SetTransformToBounds()
{
	if (BoundsSourceActor.IsValid())
	{
		// Calculate the bounds in our local rotation space translated to the BoundsSourceActor center
		const FQuat TargetRotation = GetOwner()->ActorToWorld().GetRotation();
		const FBox InitialBoundingBox = BoundsSourceActor->GetComponentsBoundingBox(bUseCollidingComponentsForSourceBounds);

		FTransform LocalTransform;
		LocalTransform.SetComponents(TargetRotation, InitialBoundingBox.GetCenter(), FVector::OneVector);
		FTransform WorldToLocal = LocalTransform.Inverse();

		FBox BoundBox(ForceInit);
		for (const UActorComponent* Component : BoundsSourceActor->GetComponents())
		{
			// Only gather visual components in the bounds calculation
			const UPrimitiveComponent* PrimitiveComponent = Cast<const UPrimitiveComponent>(Component);
			if (PrimitiveComponent != nullptr && PrimitiveComponent->IsRegistered())
			{
				const FTransform ComponentToActor = PrimitiveComponent->GetComponentTransform() * WorldToLocal;
				FBoxSphereBounds LocalSpaceComponentBounds = PrimitiveComponent->CalcBounds(ComponentToActor);
				if (LocalSpaceComponentBounds.GetBox().GetVolume() > 0.f)
				{
					BoundBox += LocalSpaceComponentBounds.GetBox();
				}
			}
		}

		// Create transform from bounds
		FVector Origin;
		FVector Extent;
		BoundBox.GetCenterAndExtents(Origin, Extent);

		Origin = LocalTransform.TransformPosition(Origin);

		FVector OwnExtent = bKeepOwnBoundsScale ? GetOwner()->CalculateComponentsBoundingBoxInLocalSpace(bUseCollidingComponentsForOwnBounds).GetExtent() : FVector::OneVector;

		FVector NewScale = GetOwner()->ActorToWorld().GetScale3D();
		if (bCopyXBounds)
		{
			NewScale.X = Extent.X / FMath::Max<double>(OwnExtent.X, UE_SMALL_NUMBER);
		}
		if (bCopyYBounds)
		{
			NewScale.Y = Extent.Y / FMath::Max<double>(OwnExtent.Y, UE_SMALL_NUMBER);
		}
		if (bCopyZBounds)
		{
			NewScale.Z = Extent.Z / FMath::Max<double>(OwnExtent.Z, UE_SMALL_NUMBER);
		}
		FTransform Transform;
		Transform.SetComponents(TargetRotation, Origin, NewScale);
		Transform = PostTransform * Transform;

		// Apply final result and notify the parent actor
		GetOwner()->Modify();
		GetOwner()->SetActorTransform(Transform);
		GetOwner()->PostEditMove(true);
	}
}

#endif

