// Copyright Epic Games, Inc. All Rights Reserved.


#include "Components/BoxComponent.h"
#include "CollisionShape.h"
#include "PhysicsEngine/BodySetup.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "SceneManagement.h"
#include "PhysicsEngine/BoxElem.h"
#include "PrimitiveSceneProxy.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoxComponent)

UBoxComponent::UBoxComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, ShowFlags(ESFIM_All0)
#endif // WITH_EDITOR
{
	BoxExtent = FVector(32.0f, 32.0f, 32.0f);

	bUseEditorCompositing = true;
}


void UBoxComponent::SetBoxExtent(FVector NewBoxExtent, bool bUpdateOverlaps)
{
	BoxExtent = NewBoxExtent;
	UpdateBounds();
	MarkRenderStateDirty();
	UpdateBodySetup();

	// do this if already created
	// otherwise, it hasn't been really created yet
	if (bPhysicsStateCreated)
	{
		// Update physics engine collision shapes
		BodyInstance.UpdateBodyScale(GetComponentTransform().GetScale3D(), true);

		if ( bUpdateOverlaps && IsCollisionEnabled() && GetOwner() )
		{
			UpdateOverlaps();
		}
	}
}

template <EShapeBodySetupHelper UpdateBodySetupAction, typename BodySetupType>
bool InvalidateOrUpdateBoxBodySetup(BodySetupType& ShapeBodySetup, bool bUseArchetypeBodySetup, FVector BoxExtent)
{
	check((bUseArchetypeBodySetup && UpdateBodySetupAction == EShapeBodySetupHelper::InvalidateSharingIfStale) || (!bUseArchetypeBodySetup && UpdateBodySetupAction == EShapeBodySetupHelper::UpdateBodySetup));
	check(ShapeBodySetup->AggGeom.BoxElems.Num() == 1);
	FKBoxElem* se = ShapeBodySetup->AggGeom.BoxElems.GetData();

	// @todo do we allow this now?
	// check for malformed values
	if (BoxExtent.X < UE_KINDA_SMALL_NUMBER)
	{
		BoxExtent.X = 1.0f;
	}

	if (BoxExtent.Y < UE_KINDA_SMALL_NUMBER)
	{
		BoxExtent.Y = 1.0f;
	}

	if (BoxExtent.Z < UE_KINDA_SMALL_NUMBER)
	{
		BoxExtent.Z = 1.0f;
	}

	float XExtent = BoxExtent.X * 2.f;
	float YExtent = BoxExtent.Y * 2.f;
	float ZExtent = BoxExtent.Z * 2.f;

	if (UpdateBodySetupAction == EShapeBodySetupHelper::UpdateBodySetup)
	{
		// now set the PhysX data values
		se->SetTransform(FTransform::Identity);
		se->X = XExtent;
		se->Y = YExtent;
		se->Z = ZExtent;
	}
	else if(se->X != XExtent || se->Y != YExtent || se->Z != ZExtent)
	{
		ShapeBodySetup = nullptr;
		bUseArchetypeBodySetup = false;
	}

	return bUseArchetypeBodySetup;
}

void UBoxComponent::UpdateBodySetup()
{
	if (PrepareSharedBodySetup<UBoxComponent>())
	{
		bUseArchetypeBodySetup = InvalidateOrUpdateBoxBodySetup<EShapeBodySetupHelper::InvalidateSharingIfStale>(ShapeBodySetup, bUseArchetypeBodySetup, BoxExtent);
	}

	CreateShapeBodySetupIfNeeded<FKBoxElem>();

	if (!bUseArchetypeBodySetup)
	{
		InvalidateOrUpdateBoxBodySetup<EShapeBodySetupHelper::UpdateBodySetup>(ShapeBodySetup, bUseArchetypeBodySetup, BoxExtent);
	}
}

#if WITH_EDITOR
void UBoxComponent::SetShowFlags(const FEngineShowFlags& InShowFlags)
{
	ShowFlags = InShowFlags;
	MarkRenderStateDirty();
}
#endif // WITH_EDITOR

bool UBoxComponent::IsZeroExtent() const
{
	return BoxExtent.IsZero();
}

FBoxSphereBounds UBoxComponent::CalcBounds(const FTransform& LocalToWorld) const 
{
	return FBoxSphereBounds( FBox( -BoxExtent, BoxExtent ) ).TransformBy(LocalToWorld);
}



FPrimitiveSceneProxy* UBoxComponent::CreateSceneProxy()
{
	/** Represents a UBoxComponent to the scene manager. */
	class FBoxSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FBoxSceneProxy(const UBoxComponent* InComponent)
			:	FPrimitiveSceneProxy(InComponent)
			,	bDrawOnlyIfSelected( InComponent->bDrawOnlyIfSelected )
			,   BoxExtents( InComponent->BoxExtent )
			,	BoxColor( InComponent->ShapeColor )
			,	LineThickness( InComponent->LineThickness )
		{
			bWillEverBeLit = false;

#if WITH_EDITOR
			struct FIterSink
			{
				FIterSink(const FEngineShowFlags InSelectedShowFlags)
					: SelectedShowFlags(InSelectedShowFlags)
				{
					SelectedShowFlagIndices.SetNum(FEngineShowFlags::SF_FirstCustom, false);
				}

				bool HandleShowFlag(uint32 InIndex, const FString& InName)
				{
					if (SelectedShowFlags.GetSingleFlag(InIndex) == true)
					{
						SelectedShowFlagIndices.PadToNum(InIndex + 1, false);
						SelectedShowFlagIndices[InIndex] = true;
					}

					return true;
				}

				bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
				{
					return HandleShowFlag(InIndex, InName);
				}

				bool OnCustomShowFlag(uint32 InIndex, const FString& InName)
				{
					return HandleShowFlag(InIndex, InName);
				}

				const FEngineShowFlags SelectedShowFlags;

				TBitArray<> SelectedShowFlagIndices;
			};

			FIterSink Sink(InComponent->ShowFlags);
			FEngineShowFlags::IterateAllFlags(Sink);
			SelectedShowFlagIndices = MoveTemp(Sink.SelectedShowFlagIndices);
#endif // WITH_EDITOR
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER( STAT_BoxSceneProxy_GetDynamicMeshElements );

			const FMatrix& LocalToWorld = GetLocalToWorld();
			
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];

					const FLinearColor DrawColor = GetViewSelectionColor(BoxColor, *View, IsSelected(), IsHovered(), false, IsIndividuallySelected() );

					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
					DrawOrientedWireBox(PDI, LocalToWorld.GetOrigin(), LocalToWorld.GetScaledAxis( EAxis::X ), LocalToWorld.GetScaledAxis( EAxis::Y ), LocalToWorld.GetScaledAxis( EAxis::Z ), BoxExtents, DrawColor, SDPG_World, LineThickness);
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			const bool bProxyVisible = !bDrawOnlyIfSelected || IsSelected();

			// Should we draw this because collision drawing is enabled, and we have collision
			const bool bShowForCollision = View->Family->EngineShowFlags.Collision && IsCollisionEnabled();

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = (IsShown(View) && bProxyVisible) || bShowForCollision;
#if WITH_EDITOR
			bool bAreAllSelectedFlagsEnabled = true;
			for (TConstSetBitIterator<> It(SelectedShowFlagIndices); It; ++It)
			{
				bAreAllSelectedFlagsEnabled &= View->Family->EngineShowFlags.GetSingleFlag(It.GetIndex());
			}

			Result.bDrawRelevance &= bAreAllSelectedFlagsEnabled;
#endif // WITH_EDITOR
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}
		virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
		uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

	private:
		const uint32	bDrawOnlyIfSelected:1;
		const FVector	BoxExtents;
		const FColor	BoxColor;
		const float		LineThickness;
#if WITH_EDITOR
		TBitArray<>		SelectedShowFlagIndices;
#endif // WITH_EDITOR
	};

	return new FBoxSceneProxy( this );
}


FCollisionShape UBoxComponent::GetCollisionShape(float Inflation) const
{
	const FVector Extent = GetScaledBoxExtent();
	return FCollisionShape::MakeBox(Extent).Inflate(Inflation);
}

