// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BrushComponent.cpp: Unreal brush component implementation
=============================================================================*/

#include "Components/BrushComponent.h"
#include "BodySetupEnums.h"
#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "UObject/Package.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Model.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "GameFramework/Volume.h"
#include "Engine/Polys.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "ActorEditorUtils.h"
#include "PrimitiveDrawingUtils.h"
#include "PhysicsEngine/BodySetup.h"
#include "DynamicMeshBuilder.h"

#if WITH_EDITOR
#include "Editor.h"
#endif //WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrushComponent)

DEFINE_LOG_CATEGORY_STATIC(LogBrushComponent, Log, All);

#if WITH_EDITORONLY_DATA
class FModelWireIndexBuffer : public FIndexBuffer
{
public:

	/** Initialization constructor. */
	FModelWireIndexBuffer(UModel* InModel):
		NumEdges(0)
	{
#if WITH_EDITOR
		if (!InModel->GetOutermost()->HasAnyPackageFlags(PKG_FilterEditorOnly))
		{
			Polys.Append(InModel->Polys->Element);
			for(int32 PolyIndex = 0;PolyIndex < InModel->Polys->Element.Num();PolyIndex++)
			{
				NumEdges += InModel->Polys->Element[PolyIndex].Vertices.Num();
			}
		}
#endif
	}

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		if(NumEdges)
		{
			const FRHIBufferCreateDesc Desc =
				FRHIBufferCreateDesc::CreateIndex<uint16>(TEXT("FModelWireIndexBuffer"), NumEdges * 2)
				.AddUsage(EBufferUsageFlags::Static)
				.SetInitActionInitializer()
				.DetermineInitialState();

			TRHIBufferInitializer<uint16> Initializer = RHICmdList.CreateBufferInitializer(Desc);

			size_t WritingIndex = 0;
			uint16 BaseIndex = 0;
			for(int32 PolyIndex = 0;PolyIndex < Polys.Num();PolyIndex++)
			{
				FPoly&	Poly = Polys[PolyIndex];
				for(int32 VertexIndex = 0; VertexIndex < Poly.Vertices.Num(); VertexIndex++)
				{
					Initializer[WritingIndex + 0] = BaseIndex + VertexIndex;
					Initializer[WritingIndex + 1] = BaseIndex + ((VertexIndex + 1) % Poly.Vertices.Num());

					WritingIndex += 2;
				}
				BaseIndex += Poly.Vertices.Num();
			}
			IndexBufferRHI = Initializer.Finalize();
		}
	}

	// Accessors.
	uint32 GetNumEdges() const { return NumEdges; }

private:
	TArray<FPoly> Polys;
	uint32 NumEdges;
};
#endif // WITH_EDITORONLY_DATA

class FBrushSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FBrushSceneProxy(UBrushComponent* Component, ABrush* Owner):
		FPrimitiveSceneProxy(Component),
#if WITH_EDITORONLY_DATA
		VertexFactory(GetScene().GetFeatureLevel(), "FBrushSceneProxy"),
		WireIndexBuffer(Component->Brush),
#endif
		bVolume(false),
		bBuilder(false),
		bSolidWhenSelected(false),
		bInManipulation(false),
		BrushColor(GEngine->C_BrushWire),
		BodySetup(Component->BrushBodySetup),
		CollisionResponse(Component->GetCollisionResponseToChannels())
	{
		bWillEverBeLit = false;

		if(Owner)
		{
			// If the editor is in a state where drawing the brush wireframe isn't desired, bail out.
			if( !GEngine->ShouldDrawBrushWireframe( Owner ) )
			{
				return;
			}

			// Determine the type of brush this is.
			bVolume = Owner->IsVolumeBrush();
			bBuilder = FActorEditorUtils::IsABuilderBrush( Owner );
			BrushColor = Owner->GetWireColor();
			bSolidWhenSelected = Owner->bSolidWhenSelected;
			bInManipulation = Owner->bInManipulation;

			// Builder brushes should be unaffected by level coloration, so if this is a builder brush, use
			// the brush color as the level color.
			if ( bBuilder )
			{
				ActorColor = BrushColor;
			}
			else
			{
				ActorColor = FActorPrimitiveColorHandler::Get().GetPrimitiveColor(Component);
			}
		}

#if WITH_EDITORONLY_DATA
		if (!Component->GetOutermost()->HasAnyPackageFlags(PKG_FilterEditorOnly))
		{
			TArray<FPoly> Polys;
			Polys.Append(Component->Brush->Polys->Element);

			TArray<FDynamicMeshVertex> OutVerts;

			for (int32 PolyIndex = 0; PolyIndex < Polys.Num(); PolyIndex++)
			{
				FPoly& Poly = Polys[PolyIndex];
				for (int32 VertexIndex = 0; VertexIndex < Poly.Vertices.Num(); VertexIndex++)
				{
					FDynamicMeshVertex Vertex;
					Vertex.Position = Poly.Vertices[VertexIndex];
					Vertex.TangentX = FVector(1, 0, 0);
					Vertex.TangentZ = FVector(0, 0, 1);
					// TangentZ.w contains the sign of the tangent basis determinant. Assume +1
					Vertex.TangentZ.Vector.W = 127;
					Vertex.TextureCoordinate[0].X = 0.0f;
					Vertex.TextureCoordinate[0].Y = 0.0f;
					OutVerts.Push(Vertex);
				}
			}

			VertexBuffers.InitFromDynamicVertex(&VertexFactory, OutVerts);
		}
#endif
	}

	virtual ~FBrushSceneProxy()
	{
#if WITH_EDITORONLY_DATA
		VertexFactory.ReleaseResource();
		WireIndexBuffer.ReleaseResource();
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
#endif

	}

	bool IsCollisionView(const FEngineShowFlags& EngineShowFlags, bool & bDrawCollision) const
	{
		const bool bInCollisionView = EngineShowFlags.CollisionVisibility || EngineShowFlags.CollisionPawn;
		if (bInCollisionView && IsCollisionEnabled())
		{
			bDrawCollision = EngineShowFlags.CollisionPawn && (CollisionResponse.GetResponse(ECC_Pawn) != ECR_Ignore);
			bDrawCollision |= EngineShowFlags.CollisionVisibility && (CollisionResponse.GetResponse(ECC_Visibility) != ECR_Ignore);
		}
		else
		{
			bDrawCollision = false;
		}

		return bInCollisionView;
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER( STAT_BrushSceneProxy_GetDynamicMeshElements );

		if( AllowDebugViewmodes() )
		{
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];

					bool bDrawCollision = false;
					const bool bInCollisionView = IsCollisionView(ViewFamily.EngineShowFlags, bDrawCollision);

					// Draw solid if 'solid when selected' and selected, or we are in a 'collision view'
					const bool bDrawSolid = ((bSolidWhenSelected && IsSelected()) || (bInCollisionView && bDrawCollision));
					// Don't draw wireframe if in a collision view mode and not drawing solid
					const bool bDrawWireframe = !bInCollisionView;

					// Choose color to draw it
					FLinearColor DrawColor = BrushColor;
					// In a collision view mode
					if(bInCollisionView)
					{
						DrawColor = BrushColor;
					}
					else if(View->Family->EngineShowFlags.ActorColoration)
					{
						DrawColor = ActorColor;
					}


					// SOLID
					if(bDrawSolid)
					{
						if(BodySetup != NULL)
						{
							auto SolidMaterialInstance = new FColoredMaterialRenderProxy(
								GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(),
								DrawColor
								);

							Collector.RegisterOneFrameMaterialProxy(SolidMaterialInstance);

							FTransform GeomTransform(GetLocalToWorld());
							BodySetup->AggGeom.GetAggGeom(GeomTransform, DrawColor.ToFColor(true), /*Material=*/SolidMaterialInstance, false, /*bSolid=*/ true, AlwaysHasVelocity(), ViewIndex, Collector);
						}
					}
					// WIREFRAME
					else if(bDrawWireframe)
					{
						// If we have the editor data (Wire Buffers) use those for wireframe
#if WITH_EDITOR
						if(WireIndexBuffer.GetNumEdges() && VertexBuffers.PositionVertexBuffer.GetNumVertices())
						{
							auto WireframeMaterial = new FColoredMaterialRenderProxy(
								GEngine->LevelColorationUnlitMaterial->GetRenderProxy(),
								GetViewSelectionColor(DrawColor, *View, !(GIsEditor && (View->Family->EngineShowFlags.Selection)) || IsSelected(), IsHovered(), false, IsIndividuallySelected() )
								);

							Collector.RegisterOneFrameMaterialProxy(WireframeMaterial);

							FMeshBatch& Mesh = Collector.AllocateMesh();
							FMeshBatchElement& BatchElement = Mesh.Elements[0];
							BatchElement.IndexBuffer = &WireIndexBuffer;
							Mesh.VertexFactory = &VertexFactory;
							Mesh.MaterialRenderProxy = WireframeMaterial;
							BatchElement.FirstIndex = 0;
							BatchElement.NumPrimitives = WireIndexBuffer.GetNumEdges();
							BatchElement.MinVertexIndex = 0;
							BatchElement.MaxVertexIndex = VertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;
							Mesh.CastShadow = false;
							Mesh.Type = PT_LineList;
							Mesh.DepthPriorityGroup = IsSelected() ? SDPG_Foreground : SDPG_World;
							Collector.AddMesh(ViewIndex, Mesh);
						}
						else 
#endif
						if(BodySetup != NULL)
							// If not, use the body setup for wireframe
						{
							FTransform GeomTransform(GetLocalToWorld());
							BodySetup->AggGeom.GetAggGeom(GeomTransform, GetSelectionColor(DrawColor, IsSelected(), IsHovered()).ToFColor(true), /* Material=*/ NULL, false, /* bSolid=*/ false, AlwaysHasVelocity(), ViewIndex, Collector);
						}

					}
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		bool bVisible = false;


		// We render volumes in collision view. In game, always, in editor, if the EngineShowFlags.Volumes option is on.
		if(bSolidWhenSelected && IsSelected())
		{
			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = true;
			Result.bDynamicRelevance = true;
			return Result;
		}

		const bool bInCollisionView = (View->Family->EngineShowFlags.Collision || View->Family->EngineShowFlags.CollisionVisibility || View->Family->EngineShowFlags.CollisionPawn);

		if(IsShown(View))
		{
			bool bNeverShow = false;

			if( GIsEditor )
			{
				const bool bShowBuilderBrush = View->Family->EngineShowFlags.BuilderBrush != 0;

				// Only render builder brush and only if the show flags indicate that we should render builder brushes.
				if( bBuilder && (!bShowBuilderBrush) )
				{
					bNeverShow = true;
				}
			}

			if(bNeverShow == false)
			{
				const bool bBSPVisible = View->Family->EngineShowFlags.BSP;
				const bool bBrushesVisible = View->Family->EngineShowFlags.Brushes;

				if ( !bVolume ) // EngineShowFlags.Collision does not apply to volumes
				{
					if( (bBSPVisible && bBrushesVisible) )
					{
						bVisible = true;
					}
				}

				// See if we should be visible because we are in a 'collision view' and have collision enabled
				if (bInCollisionView && IsCollisionEnabled())
				{
					bVisible = true;
				}

				// Always show the build brush and any brushes that are selected in the editor.
				if( GIsEditor )
				{
					if( bBuilder || IsSelected() )
					{
						bVisible = true;
					}
				}

				if ( bVolume )
				{
					const bool bVolumesVisible = View->Family->EngineShowFlags.Volumes;
					if(!GIsEditor || View->bIsGameView || bVolumesVisible)
					{
						bVisible = true;
					}
				}		
			}
		}
		
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = bVisible;
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = IsShadowCast(View);
		if(bInManipulation)
		{
			Result.bEditorNoDepthTestPrimitiveRelevance = true;
		}

		// Don't render on top in 'collision view' modes
		if(!bInCollisionView && !View->bIsGameView)
		{
			Result.bEditorPrimitiveRelevance = true;
		}

		return Result;
	}

	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override
	{
#if WITH_EDITORONLY_DATA
		WireIndexBuffer.InitResource(RHICmdList);
#endif

	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

private:
#if WITH_EDITORONLY_DATA
	FLocalVertexFactory VertexFactory;
	FModelWireIndexBuffer WireIndexBuffer;
	FStaticMeshVertexBuffers VertexBuffers;
#endif

	uint32 bVolume : 1;
	uint32 bBuilder : 1;	
	uint32 bSolidWhenSelected : 1;
	uint32 bInManipulation : 1;

	FColor BrushColor;
	FLinearColor ActorColor;

	/** Collision Response of this component**/
	UBodySetup* BodySetup;
	FCollisionResponseContainer CollisionResponse;
};

UBrushComponent::UBrushComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHiddenInGame = true;
	AlwaysLoadOnClient = false;
	AlwaysLoadOnServer = false;
	bUseAsOccluder = true;
	bUseEditorCompositing = true;
	bCanEverAffectNavigation = true;
#if WITH_EDITORONLY_DATA
	PrePivot_DEPRECATED = FVector::ZeroVector;
#endif
}

FPrimitiveSceneProxy* UBrushComponent::CreateSceneProxy()
{
	FPrimitiveSceneProxy* Proxy = NULL;
	
	if (Brush != NULL)	
	{
		// Check to make sure that we want to draw this brushed based on editor settings.
		ABrush*	Owner = Cast<ABrush>(GetOwner());
		if(Owner)
		{
			// If the editor is in a state where drawing the brush wireframe isn't desired, bail out.
			if( GEngine->ShouldDrawBrushWireframe( Owner ) )
			{
				Proxy = new FBrushSceneProxy(this, Owner);
			}
		}
		else
		{
			Proxy = new FBrushSceneProxy(this, Owner);
		}
	}

	return Proxy;
}


FBoxSphereBounds UBrushComponent::CalcBounds(const FTransform& LocalToWorld) const
{
#if WITH_EDITOR
	if(Brush && Brush->Polys && Brush->Polys->Element.Num())
	{
		TArray<FVector> Points;
		for( int32 i=0; i<Brush->Polys->Element.Num(); i++ )
		{
			for( int32 j=0; j<Brush->Polys->Element[i].Vertices.Num(); j++ )
			{
				Points.Add((FVector)Brush->Polys->Element[i].Vertices[j]);
			}
		}
		return FBoxSphereBounds( Points.GetData(), Points.Num() ).TransformBy(LocalToWorld);
	}
	else 
#endif // WITH_EDITOR
	if ((BrushBodySetup != NULL) && (BrushBodySetup->AggGeom.GetElementCount() > 0))
	{
		FBoxSphereBounds NewBounds;
		BrushBodySetup->AggGeom.CalcBoxSphereBounds(NewBounds, LocalToWorld);
		return NewBounds;
	}
	else
	{
		return FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}
}

void UBrushComponent::GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials ) const
{
#if WITH_EDITOR
	// Get the material from each polygon making up the brush.
	if( Brush && Brush->Polys )
	{
		UPolys* Polys = Brush->Polys;
		for( int32 ElementIdx = 0; ElementIdx < Polys->Element.Num(); ++ElementIdx )
		{
			OutMaterials.Add( Polys->Element[ ElementIdx ].Material );
		}
	}
#endif // WITH_EDITOR
}

void UBrushComponent::PostLoad()
{
	Super::PostLoad();

	// Stop existing BrushComponents from generating mirrored collision mesh
	if ((GetLinkerUEVersion() < VER_UE4_NO_MIRROR_BRUSH_MODEL_COLLISION) && (BrushBodySetup != NULL))
	{
		BrushBodySetup->bGenerateMirroredCollision = false;
	}

#if WITH_EDITOR
	// If loading a brush with mirroring whose body setup has not been created correctly, request that it be rebuilt now.
	// The rebuilding will actually happen in the UBodySetup::PostLoad.
	RequestUpdateBrushCollision();

	AActor* Owner = GetOwner();

	if (Owner)
	{
		AddRelativeLocation(GetComponentTransform().TransformVector(-PrePivot_DEPRECATED));
		Owner->SetPivotOffset(PrePivot_DEPRECATED);
		PrePivot_DEPRECATED = FVector::ZeroVector;
	}
#endif
}

#if WITH_EDITOR
void UBrushComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (const ABrush* BrushOwner = Cast<ABrush>(GetOwner()))
	{
		// todo: see if we can optimize to only rebuild BSP only once when all the Brushes finished changing. 
		if (BrushOwner->CanEverAffectBSP())
		{
			// if the brush changes from static to non-static or vice versa, we need to update the BSP
			if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBrushComponent, Mobility))
			{
				ConditionalRebuildAlteredBSP();
			}

			// For brush, the flag also affects navigation data export via BSP
			if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBrushComponent, bCanEverAffectNavigation))
			{
				ConditionalRebuildAlteredBSP();
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UBrushComponent::ConditionalRebuildAlteredBSP() const
{
	// Not entirely clear why we should limit the rebuilding to within a transaction,
	// but this is done in ABrush::PostEditChangeProperty, so we are following the same pattern
	if (!ABrush::GetSuppressBSPRegeneration() && GEditor && GUndo)
	{
		GEditor->RebuildAlteredBSP();
	}
}
#endif //WITH_EDITOR

ESceneDepthPriorityGroup UBrushComponent::GetStaticDepthPriorityGroup() const
{
	ABrush* BrushOwner = Cast<ABrush>(GetOwner());

	// Draw selected and builder brushes in the foreground DPG.
	if(BrushOwner && (IsOwnerSelected() || FActorEditorUtils::IsABuilderBrush(BrushOwner)))
	{
		return SDPG_Foreground;
	}
	else
	{
		return DepthPriorityGroup;
	}
}

bool UBrushComponent::IsShown(const FEngineShowFlags& ShowFlags) const
{
	if (const AActor* Actor = GetOwner())
	{
		return (Actor->IsA(AVolume::StaticClass())) ? ShowFlags.Volumes : ShowFlags.BSP;
	}

	return false;
}

#if WITH_EDITOR
bool UBrushComponent::ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	if (Brush != nullptr && Brush->Polys != nullptr)
	{
		TArray<FVector> Vertices;

		if (!bMustEncompassEntireComponent)
		{
			const bool bMustContainVertex = true;
			if (bMustContainVertex)
			{
				// Test if any poly vertex intersects the box
				for (const auto& Poly : Brush->Polys->Element)
				{
					for (const auto& Vertex : Poly.Vertices)
					{
						const FVector Location = GetComponentTransform().TransformPosition((FVector)Vertex);
						const bool bLocationIntersected = FMath::PointBoxIntersection(Location, InSelBBox);

						// If the selection box doesn't have to encompass the entire component and any poly vertex intersects with the selection
						// box, this component qualifies
						if (bLocationIntersected)
						{
							return true;
						}
					}
				}
			}
			else
			{
				// Alternative method, which can be enabled by setting bMustContainVertex = false:
				// Test if any poly edge intersects the box
				for (const auto& Poly : Brush->Polys->Element)
				{
					const int32 NumVerts = Poly.Vertices.Num();
					if (NumVerts > 0)
					{
						FVector StartVert = GetComponentTransform().TransformPosition((FVector)Poly.Vertices[NumVerts - 1]);
						for (int32 Index = 0; Index < NumVerts; ++Index)
						{
							const FVector EndVert = GetComponentTransform().TransformPosition((FVector)Poly.Vertices[Index]);

							if (FMath::LineBoxIntersection(InSelBBox, StartVert, EndVert, EndVert - StartVert))
							{
								return true;
							}

							StartVert = EndVert;
						}
					}
				}
			}

			// No poly intersected with the bounding box
			return false;
		}
		else
		{
			for (const auto& Poly : Brush->Polys->Element)
			{
				// The component must be entirely within the bounding box...
				for (const auto& Vertex : Poly.Vertices)
				{
					const FVector Location = GetComponentTransform().TransformPosition((FVector)Vertex);
					const bool bLocationIntersected = FMath::PointBoxIntersection(Location, InSelBBox);

					// If the selection box has to encompass the entire component and a poly vertex didn't intersect with the selection
					// box, this component does not qualify
					if (!bLocationIntersected)
					{
						return false;
					}
				}
			}

			// All points lay within the selection box
			return true;
		}
	}

	return false;
}


bool UBrushComponent::ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	if (Brush != nullptr && Brush->Polys != nullptr)
	{
		TArray<FVector> Vertices;

		for (const auto& Poly : Brush->Polys->Element)
		{
			for (const auto& Vertex : Poly.Vertices)
			{
				const FVector Location = GetComponentTransform().TransformPosition((FVector)Vertex);
				const bool bIntersect = InFrustum.IntersectSphere(Location, 0.0f);

				if (bIntersect && !bMustEncompassEntireComponent)
				{
					// If we intersected a vertex and we don't require the box to encompass the entire component
					// then the actor should be selected and we can stop checking
					return true;
				}
				else if (!bIntersect && bMustEncompassEntireComponent)
				{
					// If we didn't intersect a vertex but we require the box to encompass the entire component
					// then this test failed and we can stop checking
					return false;
				}
			}
		}

		// If the selection box has to encompass all of the component and none of the component's verts failed the intersection test, this component
		// is considered touching
		return true;
	}

	return false;
}

void UBrushComponent::RequestUpdateBrushCollision()
{
	if (BrushBodySetup)
	{

		const FVector LocalRelativeScale3D = GetRelativeScale3D();
		const bool bIsMirrored = (LocalRelativeScale3D.X * LocalRelativeScale3D.Y * LocalRelativeScale3D.Z) < 0.0f;
		if ((BrushBodySetup->bGenerateNonMirroredCollision && bIsMirrored) || (BrushBodySetup->bGenerateMirroredCollision && !bIsMirrored))
		{
			// Brushes only maintain one convex mesh as they can't be transformed at runtime.
			// Here we invalidate the body setup, and specify whether we wish to build a non-mirrored or a mirrored mesh.
			BrushBodySetup->bGenerateNonMirroredCollision = !bIsMirrored;
			BrushBodySetup->bGenerateMirroredCollision = bIsMirrored;
			BrushBodySetup->InvalidatePhysicsData();
		}
	}
}
#endif

void UBrushComponent::BuildSimpleBrushCollision()
{
	AActor* Owner = GetOwner();
	if(!Owner)
	{
		UE_LOGF(LogBrushComponent, Warning, "BuildSimpleBrushCollision: BrushComponent with no Owner!" );
		return;
	}

	if(BrushBodySetup == NULL)
	{
		BrushBodySetup = NewObject<UBodySetup>(this);
		check(BrushBodySetup);
	}

	// No complex collision, so use the simple for that
	BrushBodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;

#if WITH_EDITOR
	RequestUpdateBrushCollision();

	// Convert collision model into convex hulls.
	BrushBodySetup->CreateFromModel( Brush, true );

	RecreatePhysicsState();
#endif // WITH_EDITOR

	MarkPackageDirty();
}

bool UBrushComponent::IsEditorOnly() const
{
	// Default to actor component behavior instead of primitive component behavior as brush actors handle it themselves
	return bIsEditorOnly;
}

#if WITH_EDITOR
static FVector GetPolyCenter(const FPoly& Poly)
{
	FVector Result = FVector::ZeroVector;
	for (const auto& Vertex : Poly.Vertices)
	{
		Result += (FVector)Vertex;
	}

	return Result / Poly.Vertices.Num();
}

bool UBrushComponent::HasInvertedPolys() const
{
	// Determine if a brush looks as if it has had its sense inverted
	// (due to the old behavior of inverting the poly winding and normal when performing a Mirror operation).

	const FVector LocalRelativeScale3D = GetRelativeScale3D();
	const bool bIsMirrored = (LocalRelativeScale3D.X * LocalRelativeScale3D.Y * LocalRelativeScale3D.Z < 0.0f);
	// Only attempt to fix up brushes with negative scale
	if (bIsMirrored)
	{
		int NumInwardFacingPolys = 0;
		for (auto& Poly : Brush->Polys->Element)
		{
			// Calculate a nominal center point for the poly
			const FVector PolyCenter = GetPolyCenter(Poly);
			bool bIntersected = false;

			// Find intersections of a ray cast out from the center in the normal direction with the other polys
			for (auto& OtherPoly : Brush->Polys->Element)
			{
				if (&Poly != &OtherPoly)
				{
					// Calculate a nominal center point for the poly being tested for intersection
					const FVector OtherPolyCenter = GetPolyCenter(OtherPoly);
					const float Dot = FVector3f::DotProduct(Poly.Normal, OtherPoly.Normal);
					// If normals are perpendicular, skip it - this implies that the poly normal is parallel to the plane
					if (Dot != 0.0f)
					{
						const float Distance = FVector3f::DotProduct((FVector3f)OtherPolyCenter - (FVector3f)PolyCenter, OtherPoly.Normal) / Dot;
						// Only consider intersections in the direction of the poly normal
						if (Distance > 0.0f)
						{
							const FVector Intersection = PolyCenter + (FVector)Poly.Normal * Distance;

							// Does the ray intersect with the actual poly?
							if (OtherPoly.OnPoly(Intersection))
							{
								// If so, toggle the intersected flag.
								// An odd number of intersections implies an inwards facing poly.
								// An even number of intersections implies an outwards facing poly.
								bIntersected = !bIntersected;
							}
						}
					}
				}
			}

			if (bIntersected)
			{
				NumInwardFacingPolys++;
			}
		}

		// If more than half of the polys are deemed to be inwards facing, consider this to be an inside out brush
		if (NumInwardFacingPolys > Brush->Polys->Element.Num() / 2)
		{
			return true;
		}
	}

	return false;
}
#endif

