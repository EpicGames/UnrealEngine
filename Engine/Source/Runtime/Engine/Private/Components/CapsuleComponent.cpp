// Copyright Epic Games, Inc. All Rights Reserved.


#include "Components/CapsuleComponent.h"
#include "CollisionShape.h"
#include "PhysicsEngine/BodySetup.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "SceneManagement.h"
#include "PhysicsEngine/SphylElem.h"
#include "PrimitiveSceneProxy.h"
#include "SceneView.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CapsuleComponent)


UCapsuleComponent::UCapsuleComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ShapeColor = FColor(223, 149, 157, 255);

	CapsuleRadius = 22.0f;
	CapsuleHalfHeight = 44.0f;
	bUseEditorCompositing = true;
}



FPrimitiveSceneProxy* UCapsuleComponent::CreateSceneProxy()
{
	/** Represents a UCapsuleComponent to the scene manager. */
	class FDrawCylinderSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FDrawCylinderSceneProxy(const UCapsuleComponent* InComponent)
			:	FPrimitiveSceneProxy(InComponent)
			,	bDrawOnlyIfSelected( InComponent->bDrawOnlyIfSelected )
			,	CapsuleRadius( InComponent->GetScaledCapsuleRadius() )
			,	CapsuleHalfHeight( InComponent->GetScaledCapsuleHalfHeight() )
			,	ShapeColor( InComponent->ShapeColor )
			,	LineThickness( InComponent->LineThickness )
		{
			bWillEverBeLit = false;
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER( STAT_GetDynamicMeshElements_DrawDynamicElements );

		
			const FMatrix& LocalToWorld = GetLocalToWorld();
			const int32 CapsuleSides =  FMath::Clamp<int32>(CapsuleRadius/4.f, 16, 64);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{

				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					const FLinearColor DrawCapsuleColor = GetViewSelectionColor(ShapeColor, *View, IsSelected(), IsHovered(), false, IsIndividuallySelected() );

					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
					DrawWireCapsule( PDI, LocalToWorld.GetOrigin(), LocalToWorld.GetUnitAxis(EAxis::X), LocalToWorld.GetUnitAxis(EAxis::Y), LocalToWorld.GetUnitAxis(EAxis::Z), DrawCapsuleColor, CapsuleRadius, CapsuleHalfHeight, CapsuleSides, SDPG_World, LineThickness );
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			const bool bProxyVisible = !bDrawOnlyIfSelected || IsSelected();

			// Should we draw this because collision drawing is enabled, and we have collision
			const bool bShowForCollision = View->Family->EngineShowFlags.Collision && IsCollisionEnabled();

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = (IsShown(View) && bProxyVisible) || bShowForCollision;
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}
		virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
		uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

	private:
		const uint32	bDrawOnlyIfSelected:1;
		const float		CapsuleRadius;
		const float		CapsuleHalfHeight;
		const FColor	ShapeColor;
		const float		LineThickness;
	};

	return new FDrawCylinderSceneProxy( this );
}


FBoxSphereBounds UCapsuleComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FVector BoxPoint = FVector(CapsuleRadius,CapsuleRadius,CapsuleHalfHeight);
	return FBoxSphereBounds(FVector::ZeroVector, BoxPoint, CapsuleHalfHeight).TransformBy(LocalToWorld);
}

void UCapsuleComponent::CalcBoundingCylinder(float& CylinderRadius, float& CylinderHalfHeight) const 
{
	const FVector& ComponentScale = GetComponentTransform().GetScale3D();
	const float CapsuleEndCapCenter = FMath::Max(CapsuleHalfHeight - CapsuleRadius, 0.f);
	const FVector ZAxis = GetComponentTransform().TransformVectorNoScale(FVector(0.f, 0.f, CapsuleEndCapCenter * ComponentScale.Z));
	
	const float ScaledRadius = CapsuleRadius * (ComponentScale.X > ComponentScale.Y ? ComponentScale.X : ComponentScale.Y);
	
	CylinderRadius		= ScaledRadius + FMath::Sqrt(FMath::Square(ZAxis.X) + FMath::Square(ZAxis.Y));
	CylinderHalfHeight	= ScaledRadius + ZAxis.Z;
}

#if WITH_EDITORONLY_DATA
void UCapsuleComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading() && (Ar.UEVer() < VER_UE4_AFTER_CAPSULE_HALF_HEIGHT_CHANGE))
	{
		if ((CapsuleHeight_DEPRECATED != 0.0f) || (Ar.UEVer() < VER_UE4_BLUEPRINT_VARS_NOT_READ_ONLY))
		{
			CapsuleHalfHeight = CapsuleHeight_DEPRECATED;
			CapsuleHeight_DEPRECATED = 0.0f;
		}
	}
}
#endif

void UCapsuleComponent::PostLoad()
{
	Super::PostLoad();

	// Ensure this value is clamped only in the case where we're not re-running construction scripts.
	if(!GIsReconstructingBlueprintInstances)
	{
		CapsuleHalfHeight = FMath::Max3(0.f, CapsuleHalfHeight, CapsuleRadius);
	}
}

#if WITH_EDITOR
void UCapsuleComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	// We only want to modify the property that was changed at this point
	// things like propagation from CDO to instances don't work correctly if changing one property causes a different property to change
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCapsuleComponent, CapsuleHalfHeight))
	{
		CapsuleHalfHeight = FMath::Max3(0.f, CapsuleHalfHeight, CapsuleRadius);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCapsuleComponent, CapsuleRadius))
	{
		CapsuleRadius = FMath::Clamp(CapsuleRadius, 0.f, CapsuleHalfHeight);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif	// WITH_EDITOR

void UCapsuleComponent::SetCapsuleSize(float NewRadius, float NewHalfHeight, bool bUpdateOverlaps)
{
	CapsuleHalfHeight = FMath::Max3(0.f, NewHalfHeight, NewRadius);
	CapsuleRadius = FMath::Max(0.f, NewRadius);
	UpdateBounds();
	UpdateBodySetup();
	MarkRenderStateDirty();

	// do this if already created
	// otherwise, it hasn't been really created yet
	if (bPhysicsStateCreated)
	{
		// Update physics engine collision shapes
		BodyInstance.UpdateBodyScale(GetComponentTransform().GetScale3D(), true);

		if ( bUpdateOverlaps && IsCollisionEnabled() && GetOwner() )
		{
			UpdateOverlaps();
		}
	}
}


template <EShapeBodySetupHelper UpdateBodySetupAction, typename BodySetupType>
bool InvalidateOrUpdateCapsuleBodySetup(BodySetupType& ShapeBodySetup, bool bUseArchetypeBodySetup, float CapsuleRadius, float CapsuleHalfHeight)
{
	check((bUseArchetypeBodySetup && UpdateBodySetupAction == EShapeBodySetupHelper::InvalidateSharingIfStale) || (!bUseArchetypeBodySetup && UpdateBodySetupAction == EShapeBodySetupHelper::UpdateBodySetup));
	check(ShapeBodySetup->AggGeom.SphylElems.Num() == 1);
	FKSphylElem* SE = ShapeBodySetup->AggGeom.SphylElems.GetData();

	const float Length = 2 * FMath::Max(CapsuleHalfHeight - CapsuleRadius, 0.f);	//SphylElem uses height from center of capsule spheres, but UCapsuleComponent uses halfHeight from end of the sphere

	if (UpdateBodySetupAction == EShapeBodySetupHelper::UpdateBodySetup)
	{
		SE->SetTransform(FTransform::Identity);
		SE->Radius = CapsuleRadius;
		SE->Length = Length;
	}
	else
	{
		if(SE->Radius != CapsuleRadius || SE->Length != Length)
		{
			ShapeBodySetup = nullptr;
			bUseArchetypeBodySetup = false;
		}
	}
	
	return bUseArchetypeBodySetup;
}

void UCapsuleComponent::UpdateBodySetup()
{
	if (PrepareSharedBodySetup<UCapsuleComponent>())
	{
		bUseArchetypeBodySetup = InvalidateOrUpdateCapsuleBodySetup<EShapeBodySetupHelper::InvalidateSharingIfStale>(ShapeBodySetup, bUseArchetypeBodySetup, CapsuleRadius, CapsuleHalfHeight);
	}

	CreateShapeBodySetupIfNeeded<FKSphylElem>();

	if (!bUseArchetypeBodySetup)
	{
		InvalidateOrUpdateCapsuleBodySetup<EShapeBodySetupHelper::UpdateBodySetup>(ShapeBodySetup, bUseArchetypeBodySetup, CapsuleRadius, CapsuleHalfHeight);
	}
}

bool UCapsuleComponent::IsZeroExtent() const
{
	return (CapsuleRadius == 0.f) && (CapsuleHalfHeight == 0.f);
}


FCollisionShape UCapsuleComponent::GetCollisionShape(float Inflation) const
{
	const float Radius = GetScaledCapsuleRadius();
	const float HalfHeight = GetScaledCapsuleHalfHeight();
	return FCollisionShape::MakeCapsule(Radius, HalfHeight).Inflate(Inflation);
}

bool UCapsuleComponent::AreSymmetricRotations(const FQuat& A, const FQuat& B, const FVector& Scale3D) const
{
	if (Scale3D.X != Scale3D.Y)
	{
		return false;
	}

	const FVector AUp = A.GetAxisZ();
	const FVector BUp = B.GetAxisZ();
	return AUp.Equals(BUp);
}

