// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Movement.cpp: Character movement implementation

=============================================================================*/

#include "GameFramework/CharacterMovementComponent.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimInstance.h"
#include "EngineStats.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationDataInterface.h"
#include "Engine/NetConnection.h"
#include "GameFramework/WorldSettings.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "UObject/Package.h"
#include "GameFramework/PhysicsVolume.h"
#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "ObjectCacheContext.h"
#include "GameFramework/GameNetworkManager.h"
#include "GameFramework/Character.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/GameStateBase.h"
#include "Engine/Canvas.h"
#include "AI/Navigation/AvoidanceManager.h"
#include "Components/BrushComponent.h"
#include "PBDRigidsSolver.h"
#include "Chaos/ChaosUserEntity.h"
#include "Engine/NetworkObjectList.h"
#include "Net/PerfCountersHelpers.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "Runtime/Experimental/Chaos/Private/Chaos/PhysicsObjectInternal.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteExecutor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(CharacterMovementComponent)

CSV_DEFINE_CATEGORY(CharacterMovement, true);
CSV_DEFINE_CATEGORY(CharacterMovementDetailed, (!UE_BUILD_SHIPPING));

DEFINE_LOG_CATEGORY_STATIC(LogCharacterMovement, Log, All);
DEFINE_LOG_CATEGORY_STATIC(LogNavMeshMovement, Log, All);
DEFINE_LOG_CATEGORY_STATIC(LogCharacterNetSmoothing, Log, All);

/**
 * Character stats
 */
DECLARE_CYCLE_STAT(TEXT("Char Tick"), STAT_CharacterMovementTick, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char NonSimulated Time"), STAT_CharacterMovementNonSimulated, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char Simulated Time"), STAT_CharacterMovementSimulated, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char PerformMovement"), STAT_CharacterMovementPerformMovement, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char ReplicateMoveToServer"), STAT_CharacterMovementReplicateMoveToServer, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char CallServerMove"), STAT_CharacterMovementCallServerMove, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char ServerMove"), STAT_CharacterMovementServerMove, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char ServerForcePositionUpdate"), STAT_CharacterMovementForcePositionUpdate, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char RootMotionSource Calculate"), STAT_CharacterMovementRootMotionSourceCalculate, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char RootMotionSource Apply"), STAT_CharacterMovementRootMotionSourceApply, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char ClientUpdatePositionAfterServerUpdate"), STAT_CharacterMovementClientUpdatePositionAfterServerUpdate, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char CombineNetMove"), STAT_CharacterMovementCombineNetMove, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char NetSmoothCorrection"), STAT_CharacterMovementSmoothCorrection, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char SmoothClientPosition"), STAT_CharacterMovementSmoothClientPosition, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char SmoothClientPosition_Interp"), STAT_CharacterMovementSmoothClientPosition_Interp, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char SmoothClientPosition_Visual"), STAT_CharacterMovementSmoothClientPosition_Visual, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char Physics Interation"), STAT_CharPhysicsInteraction, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char StepUp"), STAT_CharStepUp, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char FindFloor"), STAT_CharFindFloor, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char AdjustFloorHeight"), STAT_CharAdjustFloorHeight, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char Update Acceleration"), STAT_CharUpdateAcceleration, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char MoveUpdateDelegate"), STAT_CharMoveUpdateDelegate, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char PhysWalking"), STAT_CharPhysWalking, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char PhysFalling"), STAT_CharPhysFalling, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char PhysNavWalking"), STAT_CharPhysNavWalking, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char NavProjectPoint"), STAT_CharNavProjectPoint, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char NavProjectLocation"), STAT_CharNavProjectLocation, STATGROUP_Character);
DECLARE_CYCLE_STAT(TEXT("Char ProcessLanded"), STAT_CharProcessLanded, STATGROUP_Character);


DECLARE_CYCLE_STAT(TEXT("Char HandleImpact"), STAT_CharHandleImpact, STATGROUP_Character);

namespace CharacterMovementConstants
{
	// MAGIC NUMBERS
	const float MAX_STEP_SIDE_Z = 0.08f;	// maximum z value for the normal on the vertical side of steps
	const float SWIMBOBSPEED = -80.f;
	const float VERTICAL_SLOPE_NORMAL_Z = 0.001f; // Slope is vertical if Abs(Normal.Z) <= this threshold. Accounts for precision problems that sometimes angle normals slightly off horizontal for vertical surface.
}

const FVector UCharacterMovementComponent::DefaultGravityDirection = FVector(0.0, 0.0, -1.0);
const float UCharacterMovementComponent::MIN_TICK_TIME = 1e-6f;
const float UCharacterMovementComponent::MIN_FLOOR_DIST = 1.9f;
const float UCharacterMovementComponent::MAX_FLOOR_DIST = 2.4f;
const float UCharacterMovementComponent::BRAKE_TO_STOP_VELOCITY = 10.f;
const float UCharacterMovementComponent::SWEEP_EDGE_REJECT_DISTANCE = 0.15f;

static const FString PerfCounter_NumServerMoves = "NumServerMoves";
static const FString PerfCounter_NumServerMoveCorrections = "NumServerMoveCorrections";

// Defines for build configs
#if DO_CHECK && !UE_BUILD_SHIPPING // Disable even if checks in shipping are enabled.
	#define devCode( Code )		checkCode( Code )
#else
	#define devCode(...)
#endif


// CVars
namespace CharacterMovementCVars
{
	// Use newer RPCs and RPC parameter serialization that allow variable length data without changing engine APIs.
	static int32 NetUsePackedMovementRPCs = 1;
	FAutoConsoleVariableRef CVarNetUsePackedMovementRPCs(
		TEXT("p.NetUsePackedMovementRPCs"),
		NetUsePackedMovementRPCs,
		TEXT("Whether to use newer movement RPC parameter packed serialization. If disabled, old deprecated movement RPCs will be used instead.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	static int32 NetPackedMovementMaxBits = 4096;
	FAutoConsoleVariableRef CVarNetPackedMovementMaxBits(
		TEXT("p.NetPackedMovementMaxBits"),
		NetPackedMovementMaxBits,
		TEXT("Max number of bits allowed in each packed movement RPC. Used to protect against bad data causing the server to allocate too much memory.\n"),
		ECVF_Default);

	// Listen server smoothing
	static int32 NetEnableListenServerSmoothing = 1;
	FAutoConsoleVariableRef CVarNetEnableListenServerSmoothing(
		TEXT("p.NetEnableListenServerSmoothing"),
		NetEnableListenServerSmoothing,
		TEXT("Whether to enable mesh smoothing on listen servers for the local view of remote clients.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	// Latent proxy prediction
	static int32 NetEnableSkipProxyPredictionOnNetUpdate = 1;
	FAutoConsoleVariableRef CVarNetEnableSkipProxyPredictionOnNetUpdate(
		TEXT("p.NetEnableSkipProxyPredictionOnNetUpdate"),
		NetEnableSkipProxyPredictionOnNetUpdate,
		TEXT("Whether to allow proxies to skip prediction on frames with a network position update, if bNetworkSkipProxyPredictionOnNetUpdate is also true on the movement component.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	static int32 EnableQueuedAnimEventsOnServer = 1;
	FAutoConsoleVariableRef CVarEnableQueuedAnimEventsOnServer(
		TEXT("a.EnableQueuedAnimEventsOnServer"),
		EnableQueuedAnimEventsOnServer,
		TEXT("Whether to enable queued anim events on servers. In most cases, when the server is doing a full anim graph update, queued notifies aren't triggered by the server, but this will enable them. Enabling this is recommended in projects using Listen Servers. \n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	// Logging when character is stuck. Off by default in shipping.
#if UE_BUILD_SHIPPING
	static float StuckWarningPeriod = -1.f;
#else
	static float StuckWarningPeriod = 1.f;
#endif

	FAutoConsoleVariableRef CVarStuckWarningPeriod(
		TEXT("p.CharacterStuckWarningPeriod"),
		StuckWarningPeriod,
		TEXT("How often (in seconds) we are allowed to log a message about being stuck in geometry.\n")
		TEXT("<0: Disable, >=0: Enable and log this often, in seconds."),
		ECVF_Default);

	static int32 NetEnableMoveCombining = 1;
	FAutoConsoleVariableRef CVarNetEnableMoveCombining(
		TEXT("p.NetEnableMoveCombining"),
		NetEnableMoveCombining,
		TEXT("Whether to enable move combining on the client to reduce bandwidth by combining similar moves.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	static int32 NetEnableMoveCombiningOnStaticBaseChange = 1;
	FAutoConsoleVariableRef CVarNetEnableMoveCombiningOnStaticBaseChange(
		TEXT("p.NetEnableMoveCombiningOnStaticBaseChange"),
		NetEnableMoveCombiningOnStaticBaseChange,
		TEXT("Whether to allow combining client moves when moving between static geometry.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	static float NetMoveCombiningAttachedLocationTolerance = 0.01f;
	FAutoConsoleVariableRef CVarNetMoveCombiningAttachedLocationTolerance(
		TEXT("p.NetMoveCombiningAttachedLocationTolerance"),
		NetMoveCombiningAttachedLocationTolerance,
		TEXT("Tolerance for relative location attachment change when combining moves. Small tolerances allow for very slight jitter due to transform updates."),
		ECVF_Default);

	static float NetMoveCombiningAttachedRotationTolerance = 0.01f;
	FAutoConsoleVariableRef CVarNetMoveCombiningAttachedRotationTolerance(
		TEXT("p.NetMoveCombiningAttachedRotationTolerance"),
		NetMoveCombiningAttachedRotationTolerance,
		TEXT("Tolerance for relative rotation attachment change when combining moves. Small tolerances allow for very slight jitter due to transform updates."),
		ECVF_Default);

	static float NetStationaryRotationTolerance = 0.1f;
	FAutoConsoleVariableRef CVarNetStationaryRotationTolerance(
		TEXT("p.NetStationaryRotationTolerance"),
		NetStationaryRotationTolerance,
		TEXT("Tolerance for GetClientNetSendDeltaTime() to remain throttled when small control rotation changes occur."),
		ECVF_Default);

	static int32 NetUseClientTimestampForReplicatedTransform = 1;
	FAutoConsoleVariableRef CVarNetUseClientTimestampForReplicatedTransform(
		TEXT("p.NetUseClientTimestampForReplicatedTransform"),
		NetUseClientTimestampForReplicatedTransform,
		TEXT("If enabled, use client timestamp changes to track the replicated transform timestamp, otherwise uses server tick time as the timestamp.\n")
		TEXT("Game session usually needs to be restarted if this is changed at runtime.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	static bool bNetServerForcePositionUpdateSyncToClient = false;
	FAutoConsoleVariableRef CVarNetServerForcePositionUpdateSyncToClient(
		TEXT("p.NetServerForcePositionUpdateSyncToClient"),
		bNetServerForcePositionUpdateSyncToClient,
		TEXT("If enabled, the server will always sync to the client timestamp when receiving a valid move after forcing updates (true is the old default behavior).\n"),
		ECVF_Default);

	static float NetServerMaxMoveDeltaTimeScalar = 1.75f;
	FAutoConsoleVariableRef CVarNetServerMaxMoveDeltaTimeScalar(
		TEXT("p.NetServerMaxMoveDeltaTimeScalar"),
		NetServerMaxMoveDeltaTimeScalar,
		TEXT("Multiplier to the MaxMoveDeltaTime allowed on the server, compared to the client. Enforced to be >= 1.0\n")
		TEXT("Allows the server to accept longer moves than the client is normally allowed to send, to avoid corrections (a value of 1 is old default behavior).\n"),
		ECVF_Default);

	static int32 ReplayLerpAcceleration = 0;
	FAutoConsoleVariableRef CVarReplayLerpAcceleration(
		TEXT( "p.ReplayLerpAcceleration" ),
		ReplayLerpAcceleration,
		TEXT( "" ),
		ECVF_Default);

	static int32 FixReplayOverSampling = 1;
	FAutoConsoleVariableRef CVarFixReplayOverSampling(
		TEXT( "p.FixReplayOverSampling" ),
		FixReplayOverSampling,
		TEXT( "If 1, remove invalid replay samples that can occur due to oversampling (sampling at higher rate than physics is being ticked)" ),
		ECVF_Default);

	int32 ForceJumpPeakSubstep = 1;
	FAutoConsoleVariableRef CVarForceJumpPeakSubstep(
		TEXT("p.ForceJumpPeakSubstep"),
		ForceJumpPeakSubstep,
		TEXT("If 1, force a jump substep to always reach the peak position of a jump, which can often be cut off as framerate lowers."),
		ECVF_Default);

	static float NetServerMoveTimestampExpiredWarningThreshold = 1.0f;
	FAutoConsoleVariableRef CVarNetServerMoveTimestampExpiredWarningThreshold(
		TEXT("net.NetServerMoveTimestampExpiredWarningThreshold"),
		NetServerMoveTimestampExpiredWarningThreshold,
		TEXT("Tolerance for ServerMove() to warn when client moves are expired more than this time threshold behind the server."),
		ECVF_Default);
		
	int32 AsyncCharacterMovement = 0;
	FAutoConsoleVariableRef CVarAsyncCharacterMovement(
		TEXT("p.AsyncCharacterMovement"),
		AsyncCharacterMovement, TEXT("1 enables asynchronous simulation of character movement on physics thread. Toggling this at runtime is not recommended. This feature is not fully developed, and its use is discouraged."));

	int32 BasedMovementMode = 2;
	FAutoConsoleVariableRef CVarBasedMovementMode(
		TEXT("p.BasedMovementMode"),
		BasedMovementMode, TEXT("0 means always on regular tick (default); 1 means only if not deferring updates; 2 means update and save based movement both on regular ticks and post physics when on a physics base."));

	/**
	 * Option to handle move acceleration as relative to dynamic movement bases. If disabled, moves are handled in world space.
	 * Enabling this will produce better motion when walking on highly dynamic / unpredictable movement bases, such as vehicles. However, there is the potential for larger corrections when landing on or jumping off if the base's rotation is not well-sync'd.
	 * This also may be a good option if using dynamic movement bases that become the player's primary frame of visual reference, such as when walking on a large boat or airship.
	 * Not compatible with deprecated move RPCs. @see NetUsePackedMovementRPCs
	 */
	static int32 NetUseBaseRelativeAcceleration = 1;
	FAutoConsoleVariableRef CVarNetUseBaseRelativeAcceleration(
		TEXT("p.NetUseBaseRelativeAcceleration"),
		NetUseBaseRelativeAcceleration,
		TEXT("If enabled, character acceleration will be treated as relative to dynamic movement bases."));

	static int32 NetUseBaseRelativeVelocity = 1;
	FAutoConsoleVariableRef CVarNetUseBaseRelativeVelocity(
		TEXT("p.NetUseBaseRelativeVelocity"),
		NetUseBaseRelativeVelocity,
		TEXT("If enabled, character velocity corrections will be treated as relative to dynamic movement bases."));

	static int32 UseTargetVelocityOnImpact = 1;
	FAutoConsoleVariableRef CVarUseTargetVelocityOnImpact(
		TEXT("p.UseTargetVelocityOnImpact"),
		UseTargetVelocityOnImpact, TEXT("When disabled, we recalculate velocity after impact by comparing our position before we moved to our position after we moved. This doesn't work correctly when colliding with physics objects, so setting this to 1 fixes this one the hit object is moving."));

	static float ClientAuthorityThresholdOnBaseChange = 0.f;
	FAutoConsoleVariableRef CVarClientAuthorityThresholdOnBaseChange(
		TEXT("p.ClientAuthorityThresholdOnBaseChange"),
		ClientAuthorityThresholdOnBaseChange,
		TEXT("When a pawn moves onto or off of a moving base, this can cause an abrupt correction. In these cases, trust the client up to this distance away from the server component location."),
		ECVF_Default);

	static float MaxFallingCorrectionLeash = 0.f;
	FAutoConsoleVariableRef CVarMaxFallingCorrectionLeash(
		TEXT("p.MaxFallingCorrectionLeash"),
		MaxFallingCorrectionLeash,
		TEXT("When airborne, some distance between the server and client locations may remain to avoid sudden corrections as clients jump from moving bases. This value is the maximum allowed distance."),
		ECVF_Default);

	static float MaxFallingCorrectionLeashBuffer = 10.f;
	FAutoConsoleVariableRef CVarMaxFallingCorrectionLeashBuffer(
		TEXT("p.MaxFallingCorrectionLeashBuffer"),
		MaxFallingCorrectionLeashBuffer,
		TEXT("To avoid constant corrections, when an airborne server and client are further than p.MaxFallingCorrectionLeash cm apart, they'll be pulled in to that distance minus this value."),
		ECVF_Default);

	static bool bAddFormerBaseVelocityToRootMotionOverrideWhenFalling = true;
	FAutoConsoleVariableRef CVarAddFormerBaseVelocityToRootMotionOverrideWhenFalling(
		TEXT("p.AddFormerBaseVelocityToRootMotionOverrideWhenFalling"),
		bAddFormerBaseVelocityToRootMotionOverrideWhenFalling,
		TEXT("To avoid sudden velocity changes when a root motion source moves the pawn from a moving base to free fall, this CVar will enable the FormerBaseVelocityDecayHalfLife property on CharacterMovementComponent."),
		ECVF_Default);

	static bool bGeometryCollectionImpulseWorkAround = true;
	FAutoConsoleVariableRef CVarGeometryCollectionImpulseWorkAround(
		TEXT("p.CVarGeometryCollectionImpulseWorkAround"),
		bGeometryCollectionImpulseWorkAround,
		TEXT("This enabled a workaround to allow impulses to be applied to geometry collection.\n"),
		ECVF_Default);

	static int32 bUseLastGoodRotationDuringCorrection = 1;
	FAutoConsoleVariableRef CVarUseLastGoodRotationDuringCorrection(
		TEXT("p.UseLastGoodRotationDuringCorrection"),
		bUseLastGoodRotationDuringCorrection,
		TEXT("When enabled, during a correction, restore the last good rotation before re-simulating saved moves if the server didn't specify one. This improves visual quality with options like bOrientToMovement or bUseControllerDesiredRotation that rotate over time."),
		ECVF_Default);

	static int32 bPreventNonVerticalOrientationBlock = 1;
	FAutoConsoleVariableRef CVarPreventNonVerticalOrientationBlock(
		TEXT("p.PreventNonVerticalOrientationBlock"),
		bPreventNonVerticalOrientationBlock,
		TEXT("When enabled, this allows a character that's supposed to remain vertical to snap to a vertical orientation even if RotationRate settings would block it. See @ShouldRemainVertical and @RotationRate."),
		ECVF_Default);

	static bool bDeferCharacterMeshMovement = false;
	FAutoConsoleVariableRef CVarDeferCharacterMeshMovement(
		TEXT("p.DeferCharacterMeshMovement"),
		bDeferCharacterMeshMovement,
		TEXT("Optimization - When enabled, defers CharacterMesh move propagation until the end of larger scoped moves. The mesh will still move, but all attached components will wait until all mesh movement is done within the scope."),
		ECVF_Default);

	static bool bDeferCharacterMeshMovementForAllCorrections = true;
	FAutoConsoleVariableRef CVarDeferCharacterMeshMovementForAllCorrections(
		TEXT("p.DeferCharacterMeshMovementForAllCorrections"),
		bDeferCharacterMeshMovementForAllCorrections,
		TEXT("Optimization - When enabled, defers CharacterMesh move propagation for all corrections until the end of larger scoped moves. Requires `bDeferCharacterMeshMovement=true'."),
		ECVF_Default);

	static bool bLedgeMovementDetectEdgeNormal = true;
	FAutoConsoleVariableRef CVarLedgeMovementDetectEdgeNormal(
		TEXT("p.LedgeMovement.DetectEdgeNormal"),
		bLedgeMovementDetectEdgeNormal,
		TEXT("Detect the normal of the ledge when avoiding walking off ledges, to try to find a better movement direction."),
		ECVF_Default);

	static bool bLedgeMovementApplyDirectMove = true;
	FAutoConsoleVariableRef CVarLedgeMovementApplyDirectMove(
		TEXT("p.LedgeMovement.ApplyDirectMove"),
		bLedgeMovementApplyDirectMove,
		TEXT("Apply the ledge movement vector directly, rather than the old method that reapplied acceleration."),
		ECVF_Default);

	static bool bEnableClientAuthScheduledPushForces = false;
	FAutoConsoleVariableRef CVarEnableClientAuthScheduledPushForces(
		TEXT("np2.CMC.EnableClientAuthScheduledPushForces"),
		bEnableClientAuthScheduledPushForces,
		TEXT("Enable a client-authoritative network flow for adding forces to pushed physics objects, requires physics prediction to be enabled in the project settings. NOTE: This is not recommended for networked physics and can produce inconsistency bugs where the client adds multiple forces for a single physics tick on the server, especially if async physics is not enabled."),
		ECVF_Default);

	static bool bApplyCharacterCollisionForceAsImpulse = false;
	FAutoConsoleVariableRef CVarApplyCharacterCollisionForceAsImpulse(
		TEXT("p.ApplyCharacterCollisionForceAsImpulse"),
		bApplyCharacterCollisionForceAsImpulse,
		TEXT("Whether to apply character impact physics force as impulse when the object hit is not moving."),
		ECVF_Default);

	static bool bResetJumpStateDuringResimIfNotFalling = true;
	FAutoConsoleVariableRef CVarResetJumpStateDuringResimIfNotFalling(
		TEXT("p.ResetJumpStateDuringResimIfNotFalling"),
		bResetJumpStateDuringResimIfNotFalling,
		TEXT("If true, we'll reset jump state during a correction & resimulation if not falling (e.g. walking). This fixes a bug that locks jump input, which could occur under poor net conditions."),
		ECVF_Default);

	static bool bRestoreFloorWhenMissingBaseComponent = true;
	FAutoConsoleVariableRef CVarRestoreFloorWhenMissingBaseComponent(
		TEXT("p.RestoreFloorWhenMissingBaseComponent"),
		bRestoreFloorWhenMissingBaseComponent,
		TEXT("If true, when we have a valid floor result, but no PrimitiveComponent, try to keep the CurrentFloorValid if possible."),
		ECVF_Default);

#if CSV_PROFILER_STATS
	bool bClientRecordMovePackedRpcStatsToCsv = false;
	FAutoConsoleVariableRef CVarClientRecordMovePackedRpcStatsToCsv(
		TEXT("p.ClientRecordMovePackedRpcStatsToCsv"),
		bClientRecordMovePackedRpcStatsToCsv,
		TEXT("Whether to record MovePacked RPC stats to Csv on the client (RPC count and RPC size)\n")
		TEXT("0: Disable, 1: Record MovePacked RPC stats"),
		ECVF_Default);

	bool bClientRecordNetCorrectionDistanceToCsv = false;
	FAutoConsoleVariableRef CVarRecordNetCorrectionDistanceToCsv(
		TEXT("p.ClientRecordNetCorrectionDistanceToCsv"),
		bClientRecordNetCorrectionDistanceToCsv,
		TEXT("Whether to record net correction distances to Csv on the client. This can be useful to understand how often corrections happen and how big are they\n")
		TEXT("0: Disable, 1: Record the correction distance"),
		ECVF_Default);
#endif //CSV_PROFILER_STATS

#if !UE_BUILD_SHIPPING

	int32 NetShowCorrections = 0;
	FAutoConsoleVariableRef CVarNetShowCorrections(
		TEXT("p.NetShowCorrections"),
		NetShowCorrections,
		TEXT("Whether to draw client position corrections (red is incorrect, green is corrected).\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Cheat);

	float NetCorrectionLifetime = 4.f;
	FAutoConsoleVariableRef CVarNetCorrectionLifetime(
		TEXT("p.NetCorrectionLifetime"),
		NetCorrectionLifetime,
		TEXT("How long a visualized network correction persists.\n")
		TEXT("Time in seconds each visualized network correction persists."),
		ECVF_Cheat);

#endif // !UE_BUILD_SHIPPING


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	static float NetForceClientAdjustmentPercent = 0.f;
	FAutoConsoleVariableRef CVarNetForceClientAdjustmentPercent(
		TEXT("p.NetForceClientAdjustmentPercent"),
		NetForceClientAdjustmentPercent,
		TEXT("Percent of ServerCheckClientError checks to return true regardless of actual error.\n")
		TEXT("Useful for testing client correction code.\n")
		TEXT("<=0: Disable, 0.05: 5% of checks will return failed, 1.0: Always send client adjustments"),
		ECVF_Cheat);

	static float NetForceClientServerMoveLossPercent = 0.f;
	FAutoConsoleVariableRef CVarNetForceClientServerMoveLossPercent(
		TEXT("p.NetForceClientServerMoveLossPercent"),
		NetForceClientServerMoveLossPercent,
		TEXT("Percent of ServerMove calls for client to not send.\n")
		TEXT("Useful for testing server force correction code.\n")
		TEXT("<=0: Disable, 0.05: 5% of checks will return failed, 1.0: never send server moves"),
		ECVF_Cheat);

	static float NetForceClientServerMoveLossDuration = 0.f;
	FAutoConsoleVariableRef CVarNetForceClientServerMoveLossDuration(
		TEXT("p.NetForceClientServerMoveLossDuration"),
		NetForceClientServerMoveLossDuration,
		TEXT("Duration in seconds for client to drop ServerMove calls when NetForceClientServerMoveLossPercent check passes.\n")
		TEXT("Useful for testing server force correction code.\n")
		TEXT("Duration of zero means single frame loss."),
		ECVF_Cheat);

	static int32 VisualizeMovement = 0;
	FAutoConsoleVariableRef CVarVisualizeMovement(
		TEXT("p.VisualizeMovement"),
		VisualizeMovement,
		TEXT("Whether to draw in-world debug information for character movement.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Cheat);

	static int32 NetVisualizeSimulatedCorrections = 0;
	FAutoConsoleVariableRef CVarNetVisualizeSimulatedCorrections(
		TEXT("p.NetVisualizeSimulatedCorrections"),
		NetVisualizeSimulatedCorrections,
		TEXT("")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Cheat);

	static int32 DebugTimeDiscrepancy = 0;
	FAutoConsoleVariableRef CVarDebugTimeDiscrepancy(
		TEXT("p.DebugTimeDiscrepancy"),
		DebugTimeDiscrepancy,
		TEXT("Whether to log detailed Movement Time Discrepancy values for testing")
		TEXT("0: Disable, 1: Enable Detection logging, 2: Enable Detection and Resolution logging"),
		ECVF_Cheat);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

extern ENGINE_API FString GPlayInEditorContextString;

/**
 * Helper to change mesh bone updates within a scope.
 * Example usage:
 *	{
 *		FScopedPreventMeshBoneUpdate ScopedNoMeshBoneUpdate(CharacterOwner->GetMesh(), EKinematicBonesUpdateToPhysics::SkipAllBones);
 *		// Do something to move mesh, bones will not update
 *		// Do something again to move mesh, bones will not update
 *	}
 *	// Movement of mesh at this point will use previous setting.
 */
struct FScopedMeshBoneUpdateOverride
{
	FScopedMeshBoneUpdateOverride(USkeletalMeshComponent* Mesh, EKinematicBonesUpdateToPhysics::Type OverrideSetting)
	: MeshRef(Mesh)
	{
		if (MeshRef)
		{
			// Save current state.
			SavedUpdateSetting = MeshRef->KinematicBonesUpdateType;
			// Override bone update setting.
			MeshRef->KinematicBonesUpdateType = OverrideSetting;
		}
	}

	~FScopedMeshBoneUpdateOverride()
	{
		if (MeshRef)
		{
			// Restore bone update flag.
			MeshRef->KinematicBonesUpdateType = SavedUpdateSetting;
		}
	}

private:
	USkeletalMeshComponent* MeshRef;
	EKinematicBonesUpdateToPhysics::Type SavedUpdateSetting;
};


/**
 * Helper to change updated component transform updates within a scope, optionally allowing it to revert back without side effects.
 * Example usage:
 *	{
 *		FScopedCapsuleMovementUpdate ScopedDeferredUpdate(UpdatedComponent, bDeferredUpdatesEnabled);
 *		// Do something to move capsule
 *		// Do something again to move capsule
 *	}
 *	// Movement of mesh at this point will use previous setting.
 */
struct FScopedCapsuleMovementUpdate : public FScopedMovementUpdate
{
	typedef FScopedMovementUpdate Super;

	FScopedCapsuleMovementUpdate(USceneComponent* UpdatedComponent, bool bEnabled)
	: Super(bEnabled ? UpdatedComponent : nullptr, EScopedUpdate::DeferredUpdates)
	{
	}
};


/**
 * Similar to FScopedCapsuleMovementUpdate, but intended for the character mesh instead.
 * @see FScopedCapsuleMovementUpdate
 */
struct FScopedMeshMovementUpdate
{
	FScopedMeshMovementUpdate(USkeletalMeshComponent* Mesh, bool bEnabled = true)
	: ScopedMoveUpdate(bEnabled && CharacterMovementCVars::bDeferCharacterMeshMovement ? Mesh : nullptr, EScopedUpdate::DeferredUpdates)
	{
	}

private:
	FScopedMovementUpdate ScopedMoveUpdate;
};



void FFindFloorResult::SetFromSweep(const FHitResult& InHit, const float InSweepFloorDist, const bool bIsWalkableFloor)
{
	bBlockingHit = InHit.IsValidBlockingHit();
	bWalkableFloor = bIsWalkableFloor;
	bLineTrace = false;
	FloorDist = InSweepFloorDist;
	LineDist = 0.f;
	HitResult = InHit;
}

void FFindFloorResult::SetFromLineTrace(const FHitResult& InHit, const float InSweepFloorDist, const float InLineDist, const bool bIsWalkableFloor)
{
	// We require a sweep that hit if we are going to use a line result.
	check(HitResult.bBlockingHit);
	if (HitResult.bBlockingHit && InHit.bBlockingHit)
	{
		// Override most of the sweep result with the line result, but save some values
		FHitResult OldHit(HitResult);
		HitResult = InHit;

		// Restore some of the old values. We want the new normals and hit actor, however.
		HitResult.Time = OldHit.Time;
		HitResult.ImpactPoint = OldHit.ImpactPoint;
		HitResult.Location = OldHit.Location;
		HitResult.TraceStart = OldHit.TraceStart;
		HitResult.TraceEnd = OldHit.TraceEnd;

		bLineTrace = true;
		FloorDist = InSweepFloorDist;
		LineDist = InLineDist;
		bWalkableFloor = bIsWalkableFloor;
	}
}

void FCharacterMovementComponentPostPhysicsTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	FActorComponentTickFunction::ExecuteTickHelper(Target, /*bTickInEditor=*/ false, DeltaTime, TickType, [this](float DilatedTime)
	{
		Target->PostPhysicsTickComponent(DilatedTime, *this);
	});
}

FString FCharacterMovementComponentPostPhysicsTickFunction::DiagnosticMessage()
{
	return Target->GetFullName() + TEXT("[UCharacterMovementComponent::PostPhysicsTick]");
}

FName FCharacterMovementComponentPostPhysicsTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		return FName(*FString::Printf(TEXT("CharacterMovementComponentPostPhysicsTick/%s"), *GetFullNameSafe(Target)));
	}

	return FName("CharacterMovementComponentPostPhysicsTick");
}

void FCharacterMovementComponentPrePhysicsTickFunction::ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	FActorComponentTickFunction::ExecuteTickHelper(Target, /*bTickInEditor=*/ false, DeltaTime, TickType, [this](float DilatedTime)
		{
			Target->PrePhysicsTickComponent(DilatedTime, *this);
		});
}

FString FCharacterMovementComponentPrePhysicsTickFunction::DiagnosticMessage()
{
	return Target->GetFullName() + TEXT("[UCharacterMovementComponent::PrePhysicsTick]");
}

FName FCharacterMovementComponentPrePhysicsTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		return FName(*FString::Printf(TEXT("CharacterMovementComponentPrePhysicsTick/%s"), *GetFullNameSafe(Target)));
	}

	return FName("CharacterMovementComponentPrePhysicsTick");
}

UCharacterMovementComponent::UCharacterMovementComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RandomStream.Initialize(FApp::bUseFixedSeed ? GetFName() : NAME_None);

	PostPhysicsTickFunction.bCanEverTick = true;
	PostPhysicsTickFunction.bStartWithTickEnabled = false;
	PostPhysicsTickFunction.SetTickFunctionEnable(false);
	PostPhysicsTickFunction.TickGroup = TG_PostPhysics;
	PostPhysicsTickFunction.bRunTransactionally = true;
	
	if (CharacterMovementCVars::AsyncCharacterMovement == 1)
	{
		PostPhysicsTickFunction.bStartWithTickEnabled = true;
		PostPhysicsTickFunction.SetTickFunctionEnable(true);
		
		PrePhysicsTickFunction.bCanEverTick = true;
		PrePhysicsTickFunction.bStartWithTickEnabled = true;
		PrePhysicsTickFunction.SetTickFunctionEnable(true);
		PrePhysicsTickFunction.TickGroup = TG_PrePhysics;
	}
	bDontFallBelowJumpZVelocityDuringJump = true;
	bApplyGravityWhileJumping = true;

	GravityScale = 1.f;
	GroundFriction = 8.0f;
	JumpZVelocity = 420.0f;
	JumpOffJumpZFactor = 0.5f;
	RotationRate = FRotator(0.f, 360.0f, 0.0f);
	SetWalkableFloorZ(0.71f);

	GravityDirection = DefaultGravityDirection;
	WorldToGravityTransform = FQuat::Identity;
	GravityToWorldTransform =  FQuat::Identity;
	bHasCustomGravity = false;

	MaxStepHeight = 45.0f;
	PerchRadiusThreshold = 0.0f;
	PerchAdditionalHeight = 40.f;

	MaxFlySpeed = 600.0f;
	MaxWalkSpeed = 600.0f;
	MaxSwimSpeed = 300.0f;
	MaxCustomMovementSpeed = MaxWalkSpeed;
	
	MaxSimulationTimeStep = 0.05f;
	MaxSimulationIterations = 8;
	MaxJumpApexAttemptsPerSimulation = 2;
	NumJumpApexAttempts = 0;

	MaxDepenetrationWithGeometry = 500.f;
	MaxDepenetrationWithGeometryAsProxy = 100.f;
	MaxDepenetrationWithPawn = 100.f;
	MaxDepenetrationWithPawnAsProxy = 2.f;

	// Set to match EVectorQuantization::RoundTwoDecimals
	NetProxyShrinkRadius = 0.01f;
	NetProxyShrinkHalfHeight = 0.01f;

	NetworkSimulatedSmoothLocationTime = 0.100f;
	NetworkSimulatedSmoothRotationTime = 0.050f;
	ListenServerNetworkSimulatedSmoothLocationTime = 0.040f;
	ListenServerNetworkSimulatedSmoothRotationTime = 0.033f;
	NetworkMaxSmoothUpdateDistance = 256.f;
	NetworkNoSmoothUpdateDistance = 384.f;
	NetworkSmoothingMode = ENetworkSmoothingMode::Exponential;
	ServerLastClientGoodMoveAckTime = -1.f;
	ServerLastClientAdjustmentTime = -1.f;
	NetworkMinTimeBetweenClientAckGoodMoves = 0.10f;
	NetworkMinTimeBetweenClientAdjustments = 0.10f;
	NetworkMinTimeBetweenClientAdjustmentsLargeCorrection = 0.05f;
	NetworkLargeClientCorrectionDistance = 15.0f;

	MaxWalkSpeedCrouched = MaxWalkSpeed * 0.5f;
	MaxOutOfWaterStepHeight = 40.0f;
	OutofWaterZ = 420.0f;
	AirControl = 0.05f;
	AirControlBoostMultiplier = 2.f;
	AirControlBoostVelocityThreshold = 25.f;
	FallingLateralFriction = 0.f;
	MaxAcceleration = 2048.0f;
	BrakingFrictionFactor = 2.0f; // Historical value, 1 would be more appropriate.
	BrakingSubStepTime = 1.0f / 33.0f;
	BrakingDecelerationWalking = MaxAcceleration;
	BrakingDecelerationFalling = 0.f;
	BrakingDecelerationFlying = 0.f;
	BrakingDecelerationSwimming = 0.f;
	LedgeCheckThreshold = 4.0f;
	JumpOutOfWaterPitch = 11.25f;

#if WITH_EDITORONLY_DATA
	CrouchedSpeedMultiplier_DEPRECATED = 0.5f;
	UpperImpactNormalScale_DEPRECATED = 0.5f;
	bForceBraking_DEPRECATED = false;
#endif
	
	Mass = 100.0f;
	bJustTeleported = true;
	SetCrouchedHalfHeight(40.0f);
	Buoyancy = 1.0f;
	LastUpdateRotation = FQuat::Identity;
	LastUpdateVelocity = FVector::ZeroVector;
	PendingImpulseToApply = FVector::ZeroVector;
	PendingLaunchVelocity = FVector::ZeroVector;
	DefaultWaterMovementMode = MOVE_Swimming;
	DefaultLandMovementMode = MOVE_Walking;
	GroundMovementMode = MOVE_Walking;
	bForceNextFloorCheck = true;
	bShrinkProxyCapsule = true;
	bCanWalkOffLedges = true;
	bCanWalkOffLedgesWhenCrouching = false;
	bNetworkSmoothingComplete = true; // Initially true until we get a net update, so we don't try to smooth to an uninitialized value.
	bWantsToLeaveNavWalking = false;
	bIsNavWalkingOnServer = false;
	bSweepWhileNavWalking = true;
	bNeedsSweepWhileWalkingUpdate = false;

	bEnablePhysicsInteraction = true;
	StandingDownwardForceScale = 1.0f;
	InitialPushForceFactor = 500.0f;
	PushForceFactor = 750000.0f;
	PushForcePointZOffsetFactor = -0.75f;
	bPushForceUsingZOffset = false;
	bPushForceScaledToMass = false;
	bScalePushForceToVelocity = true;
	
	TouchForceFactor = 1.0f;
	bTouchForceScaledToMass = true;
	MinTouchForce = -1.0f;
	MaxTouchForce = 250.0f;
	RepulsionForce = 2.5f;

	bAllowPhysicsRotationDuringAnimRootMotion = false; // Old default behavior.
	bUseControllerDesiredRotation = false;

	bUseSeparateBrakingFriction = false; // Old default behavior.

	bMaintainHorizontalGroundVelocity = true;
	bImpartBaseVelocityX = true;
	bImpartBaseVelocityY = true;
	bImpartBaseVelocityZ = true;
	bImpartBaseAngularVelocity = true;
	bIgnoreClientMovementErrorChecksAndCorrection = false;
	bClientIgnoreMovementCorrections = false;
	bServerAcceptClientAuthoritativePosition = false;
	bAlwaysCheckFloor = true;

	// default character can jump, walk, and swim
	NavAgentProps.bCanJump = true;
	NavAgentProps.bCanWalk = true;
	NavAgentProps.bCanSwim = true;
	ResetMoveState();

	ClientPredictionData = NULL;
	ServerPredictionData = NULL;
	SetNetworkMoveDataContainer(DefaultNetworkMoveDataContainer);
	SetMoveResponseDataContainer(DefaultMoveResponseDataContainer);
	ServerMoveBitWriter.SetAllowResize(true);
	MoveResponseBitWriter.SetAllowResize(true);

	// This should be greater than tolerated player timeout * 2.
	MinTimeBetweenTimeStampResets = 4.f * 60.f; 
	LastTimeStampResetServerTime = 0.f;

	bEnableScopedMovementUpdates = true;
	// Disabled by default since it can be a subtle behavior change, you should opt in if you want to accept that.
	bEnableServerDualMoveScopedMovementUpdates = false;

	bRequestedMoveUseAcceleration = true;
	bUseRVOAvoidance = false;
	bUseRVOPostProcess = false;
	AvoidanceLockVelocity = FVector::ZeroVector;
	AvoidanceLockTimer = 0.0f;
	AvoidanceGroup.bGroup0 = true;
	GroupsToAvoid.Packed = 0xFFFFFFFF;
	GroupsToIgnore.Packed = 0;
	AvoidanceConsiderationRadius = 500.0f;

	OldBaseQuat = FQuat::Identity;
	OldBaseLocation = FVector::ZeroVector;

	NavMeshProjectionInterval = 0.1f;
	NavMeshProjectionInterpSpeed = 12.f;
	NavMeshProjectionHeightScaleUp = 0.67f;
	NavMeshProjectionHeightScaleDown = 1.0f;
	NavWalkingFloorDistTolerance = 10.0f;
}

void UCharacterMovementComponent::BeginPlay()
{
	Super::BeginPlay();
	
	// Only register async callback for player controlled characters
	if (CharacterOwner && CharacterOwner->IsPlayerControlled())
	{
		RegisterAsyncCallback();
	}
}

void UCharacterMovementComponent::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	const FPackageFileVersion LinkerUEVer = GetLinkerUEVersion();

	if (LinkerUEVer < VER_UE4_CHARACTER_MOVEMENT_DECELERATION)
	{
		BrakingDecelerationWalking = MaxAcceleration;
	}

	if (LinkerUEVer < VER_UE4_CHARACTER_BRAKING_REFACTOR)
	{
		// This bool used to apply walking braking in flying and swimming modes.
		if (bForceBraking_DEPRECATED)
		{
			BrakingDecelerationFlying = BrakingDecelerationWalking;
			BrakingDecelerationSwimming = BrakingDecelerationWalking;
		}
	}

	if (LinkerUEVer < VER_UE4_CHARACTER_MOVEMENT_WALKABLE_FLOOR_REFACTOR)
	{
		// Compute the walkable floor angle, since we have never done so yet.
		UCharacterMovementComponent::SetWalkableFloorZ(WalkableFloorZ);
	}

	if (LinkerUEVer < VER_UE4_DEPRECATED_MOVEMENTCOMPONENT_MODIFIED_SPEEDS)
	{
		MaxWalkSpeedCrouched = MaxWalkSpeed * CrouchedSpeedMultiplier_DEPRECATED;
		MaxCustomMovementSpeed = MaxWalkSpeed;
	}
#endif

	CharacterOwner = Cast<ACharacter>(PawnOwner);
}


#if WITH_EDITOR
void UCharacterMovementComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	if (PropertyThatChanged && PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCharacterMovementComponent, WalkableFloorAngle))
	{
		// Compute WalkableFloorZ from the Angle.
		SetWalkableFloorAngle(WalkableFloorAngle);
	}
}
#endif // WITH_EDITOR


void UCharacterMovementComponent::OnRegister()
{
	const ENetMode NetMode = GetNetMode();
	if (bUseRVOAvoidance && NetMode == NM_Client)
	{
		bUseRVOAvoidance = false;
	}

	Super::OnRegister();

#if WITH_EDITOR
	// Compute WalkableFloorZ from the WalkableFloorAngle.
	// This is only to respond to changes propagated by PostEditChangeProperty, so it's only done in the editor.
	SetWalkableFloorAngle(WalkableFloorAngle);
#endif

	// Force linear smoothing for replays.
	const UWorld* MyWorld = GetWorld();
	const bool bIsReplay = (MyWorld && MyWorld->IsPlayingReplay());

	if (bIsReplay)
	{
		NetworkSmoothingMode = ENetworkSmoothingMode::Linear;
	}
	else if (NetMode == NM_ListenServer)
	{
		// Linear smoothing works on listen servers, but makes a lot less sense under the typical high update rate.
		if (NetworkSmoothingMode == ENetworkSmoothingMode::Linear)
		{
			NetworkSmoothingMode = ENetworkSmoothingMode::Exponential;
		}
	}
}


void UCharacterMovementComponent::BeginDestroy()
{
	if (ClientPredictionData)
	{
		delete ClientPredictionData;
		ClientPredictionData = NULL;
	}
	
	if (ServerPredictionData)
	{
		delete ServerPredictionData;
		ServerPredictionData = NULL;
	}
	
	Super::BeginDestroy();
}

void UCharacterMovementComponent::Activate(bool bReset /*= false*/)
{
	Super::Activate(bReset);

	if (IsActive())
	{
		if (CharacterOwner)
		{
			CharacterOwner->UpdateMeshAllowAutonomousTickPose();
		}
	}
}

void UCharacterMovementComponent::Deactivate()
{
	NavMovementProperties.bStopMovementAbortPaths = false; // Mirrors StopMovementKeepPathing(), because Super calls StopMovement() and we want that handled differently.
	Super::Deactivate();
	if (!IsActive())
	{
		ClearAccumulatedForces();
		if (CharacterOwner)
		{
			CharacterOwner->ResetJumpState();
			CharacterOwner->UpdateMeshAllowAutonomousTickPose();
		}
	}
	NavMovementProperties.bStopMovementAbortPaths = true;
}


void UCharacterMovementComponent::SetUpdatedComponent(USceneComponent* NewUpdatedComponent)
{
	if (NewUpdatedComponent)
	{
		const ACharacter* NewCharacterOwner = Cast<ACharacter>(NewUpdatedComponent->GetOwner());
		if (NewCharacterOwner == nullptr)
		{
			UE_LOGF(LogCharacterMovement, Error, "%ls owned by %ls must update a component owned by a Character", *GetName(), *GetNameSafe(NewUpdatedComponent->GetOwner()));
			return;
		}

		// check that UpdatedComponent is a Capsule
		if (Cast<UCapsuleComponent>(NewUpdatedComponent) == nullptr)
		{
			UE_LOGF(LogCharacterMovement, Error, "%ls owned by %ls must update a capsule component", *GetName(), *GetNameSafe(NewUpdatedComponent->GetOwner()));
			return;
		}
	}

	if ( bMovementInProgress )
	{
		// failsafe to avoid crashes in CharacterMovement. 
		bDeferUpdateMoveComponent = true;
		DeferredUpdatedMoveComponent = NewUpdatedComponent;
		return;
	}
	bDeferUpdateMoveComponent = false;
	DeferredUpdatedMoveComponent = nullptr;

	USceneComponent* OldUpdatedComponent = UpdatedComponent;
	UPrimitiveComponent* OldPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
	if (IsValid(OldPrimitive) && OldPrimitive->OnComponentBeginOverlap.IsBound())
	{
		OldPrimitive->OnComponentBeginOverlap.RemoveDynamic(this, &UCharacterMovementComponent::CapsuleTouched);
	}
	
	Super::SetUpdatedComponent(NewUpdatedComponent);
	CharacterOwner = Cast<ACharacter>(PawnOwner);

	if (UpdatedComponent != OldUpdatedComponent)
	{
		ClearAccumulatedForces();
	}

	if (UpdatedComponent == nullptr)
	{
		StopActiveMovement();
	}

	const bool bValidUpdatedPrimitive = IsValid(UpdatedPrimitive);

	if (bValidUpdatedPrimitive && bEnablePhysicsInteraction)
	{
		UpdatedPrimitive->OnComponentBeginOverlap.AddUniqueDynamic(this, &UCharacterMovementComponent::CapsuleTouched);
	}

	if (bNeedsSweepWhileWalkingUpdate)
	{
		bSweepWhileNavWalking = bValidUpdatedPrimitive ? UpdatedPrimitive->GetGenerateOverlapEvents() : false;
		bNeedsSweepWhileWalkingUpdate = false;
	}

	if (bUseRVOAvoidance && IsValid(NewUpdatedComponent))
	{
		UAvoidanceManager* AvoidanceManager = GetWorld()->GetAvoidanceManager();
		if (AvoidanceManager)
		{
			AvoidanceManager->RegisterMovementComponent(this, AvoidanceWeight);
		}
	}
}

bool UCharacterMovementComponent::HasValidData() const
{
	const bool bIsValid = UpdatedComponent && IsValid(CharacterOwner);
#if ENABLE_NAN_DIAGNOSTIC
	if (bIsValid)
	{
		// NaN-checking updates
		if (Velocity.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("UCharacterMovementComponent::HasValidData() detected NaN/INF for (%s) in Velocity:\n%s"), *GetPathNameSafe(this), *Velocity.ToString());
			UCharacterMovementComponent* MutableThis = const_cast<UCharacterMovementComponent*>(this);
			MutableThis->Velocity = FVector::ZeroVector;
		}
		if (!UpdatedComponent->GetComponentTransform().IsValid())
		{
			logOrEnsureNanError(TEXT("UCharacterMovementComponent::HasValidData() detected NaN/INF for (%s) in UpdatedComponent->ComponentTransform:\n%s"), *GetPathNameSafe(this), *UpdatedComponent->GetComponentTransform().ToHumanReadableString());
		}
		if (UpdatedComponent->GetComponentRotation().ContainsNaN())
		{
			logOrEnsureNanError(TEXT("UCharacterMovementComponent::HasValidData() detected NaN/INF for (%s) in UpdatedComponent->GetComponentRotation():\n%s"), *GetPathNameSafe(this), *UpdatedComponent->GetComponentRotation().ToString());
		}
	}
#endif
	return bIsValid;
}

bool UCharacterMovementComponent::ShouldUsePackedMovementRPCs() const
{
	return CharacterMovementCVars::NetUsePackedMovementRPCs != 0;
}

FCollisionShape UCharacterMovementComponent::GetPawnCapsuleCollisionShape(const EShrinkCapsuleExtent ShrinkMode, const float CustomShrinkAmount) const
{
	FVector Extent = GetPawnCapsuleExtent(ShrinkMode, CustomShrinkAmount);
	return FCollisionShape::MakeCapsule(Extent);
}

FVector UCharacterMovementComponent::GetPawnCapsuleExtent(const EShrinkCapsuleExtent ShrinkMode, const float CustomShrinkAmount) const
{
	check(CharacterOwner);

	float Radius, HalfHeight;
	CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(Radius, HalfHeight);
	FVector CapsuleExtent(Radius, Radius, HalfHeight);

	float RadiusEpsilon = 0.f;
	float HeightEpsilon = 0.f;

	switch(ShrinkMode)
	{
	case SHRINK_None:
		return CapsuleExtent;

	case SHRINK_RadiusCustom:
		RadiusEpsilon = CustomShrinkAmount;
		break;

	case SHRINK_HeightCustom:
		HeightEpsilon = CustomShrinkAmount;
		break;
		
	case SHRINK_AllCustom:
		RadiusEpsilon = CustomShrinkAmount;
		HeightEpsilon = CustomShrinkAmount;
		break;

	default:
		UE_LOGF(LogCharacterMovement, Warning, "Unknown EShrinkCapsuleExtent in UCharacterMovementComponent::GetCapsuleExtent");
		break;
	}

	// Don't shrink to zero extent.
	const FVector::FReal MinExtent = UE_KINDA_SMALL_NUMBER * 10;
	CapsuleExtent.X = FMath::Max<FVector::FReal>(CapsuleExtent.X - RadiusEpsilon, MinExtent);
	CapsuleExtent.Y = CapsuleExtent.X;
	CapsuleExtent.Z = FMath::Max<FVector::FReal>(CapsuleExtent.Z - HeightEpsilon, MinExtent);

	return CapsuleExtent;
}

bool UCharacterMovementComponent::DoJump(bool bReplayingMoves)
{
	return DoJump(bReplayingMoves, 0.f);
}

bool UCharacterMovementComponent::DoJump(bool bReplayingMoves, float DeltaTime)
{
	if ( CharacterOwner && CharacterOwner->CanJump() )
	{
		// Don't jump if we can't move up/down.
		if (!bConstrainToPlane || !FMath::IsNearlyEqual(FMath::Abs(GetGravitySpaceZ(PlaneConstraintNormal)), 1.f))
		{
			// If first frame of DoJump, we want to always inject the initial jump velocity.
			// For subsequent frames, during the time Jump is held, it depends... 
			// bDontFallXXXX == true means we want to ensure the character's Z velocity is never less than JumpZVelocity in this period
			// bDontFallXXXX == false means we just want to leave Z velocity alone and "let the chips fall where they may" (e.g. fall properly in physics)

			// NOTE: 
			// Checking JumpCurrentCountPreJump instead of JumpCurrentCount because Character::CheckJumpInput might have
			// incremented JumpCurrentCount just before entering this function... in order to compensate for the case when
			// on the first frame of the jump, we're already in falling stage. So we want the original value before any 
			// modification here.
			// 
			const bool bFirstJump = (CharacterOwner->JumpCurrentCountPreJump == 0);

			if (bFirstJump || bDontFallBelowJumpZVelocityDuringJump)
			{
				if (HasCustomGravity())
				{
					SetGravitySpaceZ(Velocity, FMath::Max<FVector::FReal>(GetGravitySpaceZ(Velocity), JumpZVelocity));
				}
				else
				{
					Velocity.Z = FMath::Max<FVector::FReal>(Velocity.Z, JumpZVelocity);
				}
			}
			
			SetMovementMode(MOVE_Falling);
			return true;
		}
	}
	
	return false;
}

bool UCharacterMovementComponent::CanAttemptJump() const
{
	return IsJumpAllowed() &&
		   !bWantsToCrouch &&
		   (IsMovingOnGround() || IsFalling()); // Falling included for double-jump and non-zero jump hold time, but validated by character.
}


FVector UCharacterMovementComponent::GetImpartedMovementBaseVelocity() const
{
	FVector Result = FVector::ZeroVector;
	if (CharacterOwner)
	{
		const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
		
		if (MovementBaseUtility::IsDynamicBase(MovementBaseInterfaceData))
		{
			FVector BaseVelocity = MovementBaseUtility::GetMovementBaseVelocity(MovementBaseInterfaceData, CharacterOwner->GetBasedMovement().BoneName);
			
			if (bImpartBaseAngularVelocity)
			{
				const FVector CharacterBasePosition = (UpdatedComponent->GetComponentLocation() + CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleHalfHeight() * GetGravityDirection());
				const FVector BaseTangentialVel = MovementBaseUtility::GetMovementBaseTangentialVelocity(MovementBaseInterfaceData, CharacterOwner->GetBasedMovement().BoneName, CharacterBasePosition);
				BaseVelocity += BaseTangentialVel;
			}

			if (bImpartBaseVelocityX)
			{
				Result.X = BaseVelocity.X;
			}
			if (bImpartBaseVelocityY)
			{
				Result.Y = BaseVelocity.Y;
			}
			if (bImpartBaseVelocityZ)
			{
				Result.Z = BaseVelocity.Z;
			}
		}
	}
	
	return Result;
}

void UCharacterMovementComponent::Launch(FVector const& LaunchVel)
{
	if ((MovementMode != MOVE_None) && IsActive() && HasValidData())
	{
		PendingLaunchVelocity = LaunchVel;
	}
}

bool UCharacterMovementComponent::HandlePendingLaunch()
{
	if (!PendingLaunchVelocity.IsZero() && HasValidData())
	{
		Velocity = PendingLaunchVelocity;
		SetMovementMode(MOVE_Falling);
		PendingLaunchVelocity = FVector::ZeroVector;
		bForceNextFloorCheck = true;
		return true;
	}

	return false;
}

void UCharacterMovementComponent::JumpOff(AActor* MovementBaseActor)
{
	if ( !bPerformingJumpOff )
	{
		bPerformingJumpOff = true;
		if ( CharacterOwner )
		{
			const float MaxSpeed = GetMaxSpeed() * 0.85f;
			Velocity += MaxSpeed * GetBestDirectionOffActor(MovementBaseActor);
			if (ProjectToGravityFloor(Velocity).Size() > MaxSpeed)
			{
				Velocity = MaxSpeed * Velocity.GetSafeNormal();
			}

			if (HasCustomGravity())
			{
				SetGravitySpaceZ(Velocity, JumpOffJumpZFactor * JumpZVelocity);
			}
			else
			{
				Velocity.Z = JumpOffJumpZFactor * JumpZVelocity;
			}
			
			SetMovementMode(MOVE_Falling);
		}
		bPerformingJumpOff = false;
	}
}

FVector UCharacterMovementComponent::GetBestDirectionOffActor(AActor* BaseActor) const
{
	// By default, just pick a random direction.  Derived character classes can choose to do more complex calculations,
	// such as finding the shortest distance to move in based on the BaseActor's Bounding Volume.
	const float RandAngle = FMath::DegreesToRadians(GetNetworkSafeRandomAngleDegrees());
	return RotateWorldToGravity(FVector(FMath::Cos(RandAngle), FMath::Sin(RandAngle), 0.5f).GetSafeNormal());
}

float UCharacterMovementComponent::GetNetworkSafeRandomAngleDegrees() const
{
	float Angle = RandomStream.FRand() * 360.f;

	if (!IsNetMode(NM_Standalone))
	{
		// Networked game
		// Get a timestamp that is relatively close between client and server (within ping).
		FNetworkPredictionData_Server_Character const* ServerData = (HasPredictionData_Server() ? GetPredictionData_Server_Character() : NULL);
		FNetworkPredictionData_Client_Character const* ClientData = (HasPredictionData_Client() ? GetPredictionData_Client_Character() : NULL);

		float TimeStamp = Angle;
		if (ServerData)
		{
			TimeStamp = ServerData->CurrentClientTimeStamp;
		}
		else if (ClientData)
		{
			TimeStamp = ClientData->CurrentTimeStamp;
		}
		
		// Convert to degrees with a faster period.
		const float PeriodMult = 8.0f;
		Angle = TimeStamp * PeriodMult;
		Angle = FMath::Fmod(Angle, 360.f);
	}

	return Angle;
}


void UCharacterMovementComponent::SetDefaultMovementMode()
{
	// check for water volume
	if (CanEverSwim() && IsInWater())
	{
		SetMovementMode(DefaultWaterMovementMode);
	}
	else if ( !CharacterOwner || MovementMode != DefaultLandMovementMode )
	{
		const float SavedVelocityZ = GetGravitySpaceZ(Velocity);
		SetMovementMode(DefaultLandMovementMode);

		// Avoid 1-frame delay if trying to walk but walking fails at this location.
		const FMovementBaseInterfaceData* MovementBaseData = GetMovementBaseInterfaceData();
		if (MovementMode == MOVE_Walking && !(MovementBaseData && MovementBaseData->IsValid()))
		{
			SetGravitySpaceZ(Velocity, SavedVelocityZ); // Prevent temporary walking state from zeroing Z velocity.
			SetMovementMode(MOVE_Falling);
		}
	}
}

void UCharacterMovementComponent::SetGroundMovementMode(EMovementMode NewGroundMovementMode)
{
	// Enforce restriction that it's either Walking or NavWalking.
	if (NewGroundMovementMode != MOVE_Walking && NewGroundMovementMode != MOVE_NavWalking)
	{
		return;
	}

	// Set new value
	GroundMovementMode = NewGroundMovementMode;

	// Possibly change movement modes if already on ground and choosing the other ground mode.
	const bool bOnGround = (MovementMode == MOVE_Walking || MovementMode == MOVE_NavWalking);
	if (bOnGround && MovementMode != NewGroundMovementMode)
	{
		SetMovementMode(NewGroundMovementMode);
	}
}

void UCharacterMovementComponent::SetMovementMode(EMovementMode NewMovementMode, uint8 NewCustomMode)
{
	if (NewMovementMode != MOVE_Custom)
	{
		NewCustomMode = 0;
	}

	// If trying to use NavWalking but there is no navmesh, use walking instead.
	if (NewMovementMode == MOVE_NavWalking)
	{
		if (GetNavData() == nullptr)
		{
			NewMovementMode = MOVE_Walking;
		}
	}

	// Do nothing if nothing is changing.
	if (MovementMode == NewMovementMode)
	{
		// Allow changes in custom sub-mode.
		if ((NewMovementMode != MOVE_Custom) || (NewCustomMode == CustomMovementMode))
		{
			return;
		}
	}

	const EMovementMode PrevMovementMode = MovementMode;
	const uint8 PrevCustomMode = CustomMovementMode;

	MovementMode = NewMovementMode;
	CustomMovementMode = NewCustomMode;

	// We allow setting movement mode before we have a component to update, in case this happens at startup.
	if (!HasValidData())
	{
		return;
	}
	
	// Handle change in movement mode
	OnMovementModeChanged(PrevMovementMode, PrevCustomMode);

	// @todo do we need to disable ragdoll physics here? Should this function do nothing if in ragdoll?

	bMovementModeDirty = true; // lets async callback know movement mode was dirtied on game thread
}


void UCharacterMovementComponent::OnMovementModeChanged(EMovementMode PreviousMovementMode, uint8 PreviousCustomMode)
{
	if (!HasValidData())
	{
		return;
	}

	// Update collision settings if needed
	if (MovementMode == MOVE_NavWalking)
	{
		// Reset cached nav location used by NavWalking
		CachedNavLocation = FNavLocation();

		GroundMovementMode = MovementMode;
		// Walking uses only XY velocity
		Velocity = ProjectToGravityFloor(Velocity);
		SetNavWalkingPhysics(true);
	}
	else if (PreviousMovementMode == MOVE_NavWalking)
	{
		if (MovementMode == DefaultLandMovementMode || IsWalking())
		{
			const bool bSucceeded = TryToLeaveNavWalking();
			if (!bSucceeded)
			{
				return;
			}
		}
		else
		{
			SetNavWalkingPhysics(false);
		}
	}

	// React to changes in the movement mode.
	if (MovementMode == MOVE_Walking)
	{
		// Walking uses only XY velocity, and must be on a walkable floor, with a Base.
		Velocity = FVector::VectorPlaneProject(Velocity, -GetGravityDirection());
		bCrouchMaintainsBaseLocation = true;
		GroundMovementMode = MovementMode;

		// make sure we update our new floor/base on initial entry of the walking physics
		FindFloor(UpdatedComponent->GetComponentLocation(), CurrentFloor, false);
		AdjustFloorHeight();
		SetBaseFromFloor(CurrentFloor);
	}
	else
	{
		CurrentFloor.Clear();
		bCrouchMaintainsBaseLocation = false;

		if (MovementMode == MOVE_Falling)
		{
			if (bStayBasedInAir == false)
			{
				ApplyImpartedMovementBaseVelocity();
			}
			CharacterOwner->Falling();
		}

		if (!IsFalling() || bStayBasedInAir == false)
		{
			FMovementBaseInterfaceData EmptyMovementBaseData;
			SetBase(&EmptyMovementBaseData);
		}

		if (MovementMode == MOVE_None)
		{
			// Kill velocity and clear queued up events
			StopMovementKeepPathing();
			CharacterOwner->ResetJumpState();
			ClearAccumulatedForces();
		}
	}

	if (MovementMode == MOVE_Falling && PreviousMovementMode != MOVE_Falling)
	{
		IPathFollowingAgentInterface* PFAgent = GetPathFollowingAgent();
		if (PFAgent)
		{
			PFAgent->OnStartedFalling();
		}
	}

	CharacterOwner->OnMovementModeChanged(PreviousMovementMode, PreviousCustomMode);
	ensureMsgf(GroundMovementMode == MOVE_Walking || GroundMovementMode == MOVE_NavWalking, TEXT("Invalid GroundMovementMode %d. MovementMode: %d, PreviousMovementMode: %d"), GroundMovementMode.GetValue(), MovementMode.GetValue(), PreviousMovementMode);
};


namespace PackedMovementModeConstants
{
	const uint32 GroundShift = FMath::CeilLogTwo(MOVE_MAX);
	const uint8 CustomModeThr = 2 * (1 << GroundShift);
	const uint8 GroundMask = (1 << GroundShift) - 1;
}

uint8 UCharacterMovementComponent::PackNetworkMovementMode() const
{
	if (MovementMode != MOVE_Custom)
	{
		ensureMsgf(GroundMovementMode == MOVE_Walking || GroundMovementMode == MOVE_NavWalking, TEXT("Invalid GroundMovementMode %d."), GroundMovementMode.GetValue());
		const uint8 GroundModeBit = (GroundMovementMode == MOVE_Walking ? 0 : 1);
		return uint8(MovementMode.GetValue()) | (GroundModeBit << PackedMovementModeConstants::GroundShift);
	}
	else
	{
		return CustomMovementMode + PackedMovementModeConstants::CustomModeThr;
	}
}


void UCharacterMovementComponent::UnpackNetworkMovementMode(const uint8 ReceivedMode, TEnumAsByte<EMovementMode>& OutMode, uint8& OutCustomMode, TEnumAsByte<EMovementMode>& OutGroundMode) const
{
	if (ReceivedMode < PackedMovementModeConstants::CustomModeThr)
	{
		OutMode = TEnumAsByte<EMovementMode>(ReceivedMode & PackedMovementModeConstants::GroundMask);
		OutCustomMode = 0;
		const uint8 GroundModeBit = (ReceivedMode >> PackedMovementModeConstants::GroundShift);
		OutGroundMode = TEnumAsByte<EMovementMode>(GroundModeBit == 0 ? MOVE_Walking : MOVE_NavWalking);
	}
	else
	{
		OutMode = MOVE_Custom;
		OutCustomMode = ReceivedMode - PackedMovementModeConstants::CustomModeThr;
		OutGroundMode = MOVE_Walking;
	}
}


void UCharacterMovementComponent::ApplyNetworkMovementMode(const uint8 ReceivedMode)
{
	TEnumAsByte<EMovementMode> NetMovementMode(MOVE_None);
	TEnumAsByte<EMovementMode> NetGroundMode(MOVE_None);
	uint8 NetCustomMode(0);
	UnpackNetworkMovementMode(ReceivedMode, NetMovementMode, NetCustomMode, NetGroundMode);
	ensureMsgf(NetGroundMode == MOVE_Walking || NetGroundMode == MOVE_NavWalking, TEXT("Invalid NetGroundMode %d."), NetGroundMode.GetValue());

	// set additional flag, GroundMovementMode will be overwritten by SetMovementMode to match actual mode on client side
	bIsNavWalkingOnServer = (NetGroundMode == MOVE_NavWalking);

	GroundMovementMode = NetGroundMode;
	SetMovementMode(NetMovementMode, NetCustomMode);
}

void UCharacterMovementComponent::PerformAirControlForPathFollowing(FVector Direction, float ZDiff)
{
	// use air control if low grav or above destination and falling towards it
	if ( CharacterOwner && GetGravitySpaceZ(Velocity) < 0.f && (ZDiff < 0.f || GetGravityZ() > 0.9f * GetWorld()->GetDefaultGravityZ()))
	{
		if ( ZDiff < 0.f )
		{
			const FVector LateralVelocity = ProjectToGravityFloor(Velocity);
			if ( LateralVelocity.SizeSquared() == 0.f )
			{
				Acceleration = FVector::ZeroVector;
			}
			else
			{
				const FVector::FReal Dist2D = ProjectToGravityFloor(Direction).Size();
				//Direction.Z = 0.f;
				Acceleration = Direction.GetSafeNormal() * GetMaxAcceleration();

				if ( (Dist2D < 0.5f * FMath::Abs(GetGravitySpaceZ(Direction))) && ((Velocity | Direction) > 0.5f*FMath::Square(Dist2D)) )
				{
					Acceleration *= -1.f;
				}

				if ( Dist2D < 1.5f*CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleRadius() )
				{
					Velocity = GetGravitySpaceComponentZ(Velocity);
					Acceleration = FVector::ZeroVector;
				}
				else if ( (Velocity | Direction) < 0.f )
				{
					float M = FMath::Max(0.f, 0.2f - GetWorld()->DeltaTimeSeconds);
					Velocity = LateralVelocity * M + GetGravitySpaceComponentZ(Velocity);
				}
			}
		}
	}
}

void UCharacterMovementComponent::Serialize(FArchive& Archive)
{
	Super::Serialize(Archive);

	if (Archive.IsLoading() && Archive.UEVer() < VER_UE4_ADDED_SWEEP_WHILE_WALKING_FLAG)
	{
		// We need to update the bSweepWhileNavWalking flag to match the previous behavior.
		// Since UpdatedComponent is transient, we'll have to wait until we're registered.
		bNeedsSweepWhileWalkingUpdate = true;
	}
	
	if (Archive.IsMigratingRemoteObjects())
	{
		bool bHasServerPredictionData = false;

		if (Archive.IsSaving())
		{
			bHasServerPredictionData = (ServerPredictionData != nullptr);
			Archive << bHasServerPredictionData;
		}
		else if (Archive.IsLoading())
		{
			Archive << bHasServerPredictionData;
			if (bHasServerPredictionData && ServerPredictionData == nullptr)
			{
				ServerPredictionData = new FNetworkPredictionData_Server_Character(*this);
			}
		}

		if (bHasServerPredictionData)
		{
			ensure(ServerPredictionData);

			ServerPredictionData->Serialize(Archive);
		}
	}
}

void UCharacterMovementComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UCharacterMovementComponent* This = CastChecked<UCharacterMovementComponent>(InThis);

	if (This->HasPredictionData_Client())
	{
		if (const FNetworkPredictionData_Client* ClientData = This->GetPredictionData_Client())
		{
			ClientData->AddStructReferencedObjects(Collector);
		}
	}
	else if (This->HasPredictionData_Server())
	{
		if (const FNetworkPredictionData_Server* ServerData = This->GetPredictionData_Server())
		{
			ServerData->AddStructReferencedObjects(Collector);
		}
	}
}

void UCharacterMovementComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	SCOPED_NAMED_EVENT(UCharacterMovementComponent_TickComponent, FColor::Yellow);
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovement);
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementTick);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(CharacterMovement);

	FVector InputVector = FVector::ZeroVector;
	bool bUsingAsyncTick = (CharacterMovementCVars::AsyncCharacterMovement == 1) && IsAsyncCallbackRegistered();
	if (!bUsingAsyncTick)
	{
		// Do not consume input if simulating asynchronously, we will consume input when filling out async inputs.
		InputVector = ConsumeInputVector();
	}

	if (!HasValidData() || ShouldSkipUpdate(DeltaTime))
	{
		return;
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// Super tick may destroy/invalidate CharacterOwner or UpdatedComponent, so we need to re-check.
	if (!HasValidData())
	{
		return;
	}

	if (bUsingAsyncTick)
	{
		check(CharacterOwner && CharacterOwner->GetMesh());
		USkeletalMeshComponent* CharacterMesh = CharacterOwner->GetMesh();
		if (CharacterMesh->ShouldTickPose())
		{
			// Keep track of if we're playing root motion, just in case the root motion montage ends this frame.
			// Also cache the root motion translation scale, in case the root motion ends in TickPose and
			// translation scale is reset by a blend out listener.
			const bool bWasPlayingRootMotion = CharacterOwner->IsPlayingRootMotion();
			const float RootMotionTranslationScale = CharacterOwner->GetAnimRootMotionTranslationScale();

			CharacterMesh->TickPose(DeltaTime, true);
			// We are simulating character movement on physics thread, do not tick movement.
			const bool bIsPlayingRootMotion = CharacterOwner->IsPlayingRootMotion();
			if (bIsPlayingRootMotion || bWasPlayingRootMotion)
			{
				FRootMotionMovementParams RootMotion = CharacterMesh->ConsumeRootMotion();
				if (RootMotion.bHasRootMotion)
				{
					RootMotion.ScaleRootMotionTranslation(RootMotionTranslationScale);
					RootMotionParams.Accumulate(RootMotion);
				}
			}
		}

		AccumulateRootMotionForAsync(DeltaTime, AsyncRootMotion);

		return;
	}

	// See if we fell out of the world.
	const bool bIsSimulatingPhysics = UpdatedComponent->IsSimulatingPhysics();
	if (CharacterOwner->GetLocalRole() == ROLE_Authority && (!bCheatFlying || bIsSimulatingPhysics) && !CharacterOwner->CheckStillInWorld())
	{
		return;
	}

	// We don't update if simulating physics (eg ragdolls).
	if (bIsSimulatingPhysics)
	{
		// Update camera to ensure client gets updates even when physics move it far away from point where simulation started
		if (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy && IsNetMode(NM_Client))
		{
			MarkForClientCameraUpdate();
		}

		ClearAccumulatedForces();
		return;
	}

	AvoidanceLockTimer -= DeltaTime;

	if (CharacterOwner->GetLocalRole() > ROLE_SimulatedProxy)
	{
		SCOPE_CYCLE_COUNTER(STAT_CharacterMovementNonSimulated);

		// If we are a client we might have received an update from the server.
		const bool bIsClient = (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy && IsNetMode(NM_Client));
		if (bIsClient)
		{
			FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
			if (ClientData && ClientData->bUpdatePosition)
			{
				ClientUpdatePositionAfterServerUpdate();
			}
		}

		// Defer all mesh child updates until all movement completes. Avoiding combining this with the corrections above, as those may cause a lot of significant movement.
		FScopedMeshMovementUpdate ScopedMeshUpdate(CharacterOwner->GetMesh());

		// Perform input-driven move for any locally-controlled character, and also
		// allow animation root motion or physics to move characters even if they have no controller
		const bool bShouldPerformControlledCharMove = CharacterOwner->IsLocallyControlled() 
													  || (!CharacterOwner->GetController() && bRunPhysicsWithNoController)		
													  || (!CharacterOwner->GetController() && CharacterOwner->IsPlayingRootMotion());

		if (bShouldPerformControlledCharMove)
		{
			ControlledCharacterMove(InputVector, DeltaTime);

			const bool bIsaListenServerAutonomousProxy = CharacterOwner->IsLocallyControlled()
													 	 && (CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy);

			if (bIsaListenServerAutonomousProxy)
			{
				ServerAutonomousProxyTick(DeltaTime);
			}
		}
		else if (CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy)
		{
			// Server ticking for remote client.
			// Between net updates from the client we need to update position if based on another object,
			// otherwise the object will move on intermediate frames and we won't follow it.
			MaybeUpdateBasedMovement(DeltaTime);
			MaybeSaveBaseLocation();

			ServerAutonomousProxyTick(DeltaTime);

			// Smooth on listen server for local view of remote clients. We may receive updates at a rate different than our own tick rate.
			if (CharacterMovementCVars::NetEnableListenServerSmoothing && !bNetworkSmoothingComplete && IsNetMode(NM_ListenServer))
			{
				SmoothClientPosition(DeltaTime);
			}
		}
	}
	else if (CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy)
	{
		// Defer all mesh child updates until all movement completes.
		FScopedMeshMovementUpdate ScopedMeshUpdate(CharacterOwner->GetMesh());

		if (bShrinkProxyCapsule)
		{
			AdjustProxyCapsuleSize();
		}
		SimulatedTick(DeltaTime);
	}

	if (bUseRVOAvoidance)
	{
		UpdateDefaultAvoidance();
	}

	if (bEnablePhysicsInteraction)
	{
		SCOPE_CYCLE_COUNTER(STAT_CharPhysicsInteraction);
		ApplyDownwardForce(DeltaTime);
		ApplyRepulsionForce(DeltaTime);
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const bool bVisualizeMovement = CharacterMovementCVars::VisualizeMovement > 0;
	if (bVisualizeMovement)
	{
		VisualizeMovement();
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

}

void UCharacterMovementComponent::PrePhysicsTickComponent(float DeltaTime, FCharacterMovementComponentPrePhysicsTickFunction& ThisTickFunction)
{
	BuildAsyncInput();
}

void UCharacterMovementComponent::PostPhysicsTickComponent(float DeltaTime, FCharacterMovementComponentPostPhysicsTickFunction& ThisTickFunction)
{
	ProcessAsyncOutput();

	if (bDeferUpdateBasedMovement)
	{
		if(CharacterMovementCVars::AsyncCharacterMovement == 1)
		{
			ensure(false); // Not supported
		}
		FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, bEnableScopedMovementUpdates);
		UpdateBasedMovement(DeltaTime);
		SaveBaseLocation();
		bDeferUpdateBasedMovement = false;
	}
}

void UCharacterMovementComponent::AdjustProxyCapsuleSize()
{
	if (bShrinkProxyCapsule && CharacterOwner && CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy && CharacterOwner->GetCapsuleComponent() != nullptr)
	{
		bShrinkProxyCapsule = false;

		float ShrinkRadius = FMath::Max(0.f, NetProxyShrinkRadius);
		float ShrinkHalfHeight = FMath::Max(0.f, NetProxyShrinkHalfHeight);

		if (ShrinkRadius == 0.f && ShrinkHalfHeight == 0.f)
		{
			return;
		}

		float Radius, HalfHeight;
		CharacterOwner->GetCapsuleComponent()->GetUnscaledCapsuleSize(Radius, HalfHeight);
		const float ComponentScale = CharacterOwner->GetCapsuleComponent()->GetShapeScale();

		if (ComponentScale <= UE_KINDA_SMALL_NUMBER)
		{
			return;
		}

		const float NewRadius = FMath::Max(0.f, Radius - ShrinkRadius / ComponentScale);
		const float NewHalfHeight = FMath::Max(0.f, HalfHeight - ShrinkHalfHeight / ComponentScale);

		if (NewRadius == 0.f || NewHalfHeight == 0.f)
		{
			UE_LOGF(LogCharacterMovement, Warning, "Invalid attempt to shrink Proxy capsule for %ls to zero dimension!", *CharacterOwner->GetName());
			return;
		}

		UE_LOGF(LogCharacterMovement, Verbose, "Shrinking capsule for %ls from (r=%.3f, h=%.3f) to (r=%.3f, h=%.3f)", *CharacterOwner->GetName(),
			Radius * ComponentScale, HalfHeight * ComponentScale, NewRadius * ComponentScale, NewHalfHeight * ComponentScale);

		CharacterOwner->GetCapsuleComponent()->SetCapsuleSize(NewRadius, NewHalfHeight, true);
	}	
}


void UCharacterMovementComponent::SimulatedTick(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementSimulated);
	checkSlow(CharacterOwner != nullptr);

	// If we are playing a RootMotion AnimMontage.
	if (CharacterOwner->IsPlayingNetworkedRootMotionMontage())
	{
		bWasSimulatingRootMotion = true;
		UE_LOGF(LogRootMotion, Verbose, "UCharacterMovementComponent::SimulatedTick");

		// Tick animations before physics.
		if( CharacterOwner && CharacterOwner->GetMesh() )
		{
			TickCharacterPose(DeltaSeconds);

			// Make sure animation didn't trigger an event that destroyed us
			if (!HasValidData())
			{
				return;
			}
		}

		const FQuat OldRotationQuat = UpdatedComponent->GetComponentQuat();
		const FVector OldLocation = UpdatedComponent->GetComponentLocation();

		USkeletalMeshComponent* Mesh = CharacterOwner->GetMesh();
		const FVector SavedMeshRelativeLocation = Mesh ? Mesh->GetRelativeLocation() : FVector::ZeroVector;

		if( RootMotionParams.bHasRootMotion )
		{
			SimulateRootMotion(DeltaSeconds, RootMotionParams.GetRootMotionTransform());

#if !(UE_BUILD_SHIPPING)
			// debug
			#if 0
			if (CharacterOwner && false)
			{
				const FRotator OldRotation = OldRotationQuat.Rotator();
				const FRotator NewRotation = UpdatedComponent->GetComponentRotation();
				const FVector NewLocation = UpdatedComponent->GetComponentLocation();
				DrawDebugCoordinateSystem(GetWorld(), CharacterOwner->GetMesh()->GetComponentLocation() + FVector(0,0,1), NewRotation, 50.f, false);
				DrawDebugLine(GetWorld(), OldLocation, NewLocation, FColor::Red, false, 10.f);

				const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
				UE_LOGF(LogRootMotion, Log,  "UCharacterMovementComponent::SimulatedTick DeltaMovement Translation: %ls, Rotation: %ls, MovementBase: %ls",
					*(NewLocation - OldLocation).ToCompactString(), *(NewRotation - OldRotation).GetNormalized().ToCompactString(), *GetNameSafe(MovementBaseInterfaceData ? MovementBaseInterfaceData->GetMovementBaseObject() : nullptr));
			}
			#endif
#endif // !(UE_BUILD_SHIPPING)
		}

		// then, once our position is up to date with our animation, 
		// handle position correction if we have any pending updates received from the server.
		if( CharacterOwner && (CharacterOwner->RootMotionRepMoves.Num() > 0) )
		{
			CharacterOwner->SimulatedRootMotionPositionFixup(DeltaSeconds);
		}

		if (!bNetworkSmoothingComplete && (NetworkSmoothingMode == ENetworkSmoothingMode::Linear))
		{
			// Same mesh with different rotation?
			const FQuat NewCapsuleRotation = UpdatedComponent->GetComponentQuat();
			if (Mesh == CharacterOwner->GetMesh() && !NewCapsuleRotation.Equals(OldRotationQuat, 1e-6f) && ClientPredictionData)
			{
				// Smoothing should lerp toward this new rotation target, otherwise it will just try to go back toward the old rotation.
				ClientPredictionData->MeshRotationTarget = NewCapsuleRotation;
				Mesh->SetRelativeLocationAndRotation(SavedMeshRelativeLocation, CharacterOwner->GetBaseRotationOffset());
			}
		}
	}
	else if (CurrentRootMotion.HasActiveRootMotionSources())
	{
		// We have root motion sources and possibly animated root motion
		bWasSimulatingRootMotion = true;
		UE_LOGF(LogRootMotion, Verbose, "UCharacterMovementComponent::SimulatedTick");

		// If we have RootMotionRepMoves, find the most recent important one and set position/rotation to it
		bool bCorrectedToServer = false;
		const FVector OldLocation = UpdatedComponent->GetComponentLocation();
		const FQuat OldRotation = UpdatedComponent->GetComponentQuat();
		if( CharacterOwner->RootMotionRepMoves.Num() > 0 )
		{
			// Move Actor back to position of that buffered move. (server replicated position).
			FSimulatedRootMotionReplicatedMove& RootMotionRepMove = CharacterOwner->RootMotionRepMoves.Last();
			if( CharacterOwner->RestoreReplicatedMove(RootMotionRepMove) )
			{
				bCorrectedToServer = true;
			}
			Acceleration = RootMotionRepMove.RootMotion.Acceleration;

			CharacterOwner->PostNetReceiveVelocity(RootMotionRepMove.RootMotion.LinearVelocity);
			LastUpdateVelocity = RootMotionRepMove.RootMotion.LinearVelocity;

			// Convert RootMotionSource Server IDs -> Local IDs in AuthoritativeRootMotion and cull invalid
			// so that when we use this root motion it has the correct IDs
			ConvertRootMotionServerIDsToLocalIDs(CurrentRootMotion, RootMotionRepMove.RootMotion.AuthoritativeRootMotion, RootMotionRepMove.Time);
			RootMotionRepMove.RootMotion.AuthoritativeRootMotion.CullInvalidSources();

			// Set root motion states to that of repped in state
			CurrentRootMotion.UpdateStateFrom(RootMotionRepMove.RootMotion.AuthoritativeRootMotion, true);

			// Clear out existing RootMotionRepMoves since we've consumed the most recent
			UE_LOGF(LogRootMotion, Log,  "\tClearing old moves in SimulatedTick (%d)", CharacterOwner->RootMotionRepMoves.Num());
			CharacterOwner->RootMotionRepMoves.Reset();
		}

		// Update replicated gravity direction
		if (bNetworkGravityDirectionChanged)
		{
			SetGravityDirection(CharacterOwner->GetReplicatedGravityDirection());
			bNetworkGravityDirectionChanged = false;
		}

		// Update replicated movement mode.
		if (bNetworkMovementModeChanged)
		{
			ApplyNetworkMovementMode(CharacterOwner->GetReplicatedMovementMode());
			bNetworkMovementModeChanged = false;
		}

		// Perform movement
		PerformMovement(DeltaSeconds);

		// After movement correction, smooth out error in position if any.
		if( bCorrectedToServer || CurrentRootMotion.NeedsSimulatedSmoothing() )
		{
			SmoothCorrection(OldLocation, OldRotation, UpdatedComponent->GetComponentLocation(), UpdatedComponent->GetComponentQuat());
		}
	}
	// Not playing RootMotion AnimMontage
	else
	{
		// if we were simulating root motion, we've been ignoring regular ReplicatedMovement updates.
		// If we're not simulating root motion anymore, force us to sync our movement properties.
		// (Root Motion could leave Velocity out of sync w/ ReplicatedMovement)
		if( bWasSimulatingRootMotion )
		{
			CharacterOwner->RootMotionRepMoves.Empty();
			CharacterOwner->OnRep_ReplicatedMovement();
			CharacterOwner->OnRep_ReplicatedBasedMovement();
			SetGravityDirection(CharacterOwner->GetReplicatedGravityDirection());
			ApplyNetworkMovementMode(GetCharacterOwner()->GetReplicatedMovementMode());
		}

		if (CharacterOwner->IsReplicatingMovement() && UpdatedComponent)
		{
			USkeletalMeshComponent* Mesh = CharacterOwner->GetMesh();
			const FVector SavedMeshRelativeLocation = Mesh ? Mesh->GetRelativeLocation() : FVector::ZeroVector; 
			const FQuat SavedCapsuleRotation = UpdatedComponent->GetComponentQuat();
			const bool bPreventMeshMovement = !bNetworkSmoothingComplete;

			// Avoid moving the mesh during movement if SmoothClientPosition will take care of it.
			{
				const FScopedPreventAttachedComponentMove PreventMeshMovement(bPreventMeshMovement ? Mesh : nullptr);
				if (CharacterOwner->IsPlayingRootMotion())
				{
					// Update replicated gravity direction
					if (bNetworkGravityDirectionChanged)
					{
						SetGravityDirection(CharacterOwner->GetReplicatedGravityDirection());
						bNetworkGravityDirectionChanged = false;
					}

					// Update replicated movement mode.
					if (bNetworkMovementModeChanged)
					{
						ApplyNetworkMovementMode(CharacterOwner->GetReplicatedMovementMode());
						bNetworkMovementModeChanged = false;
					}

					PerformMovement(DeltaSeconds);
				}
				else
				{
					SimulateMovement(DeltaSeconds);
				}
			}

			// With Linear smoothing we need to know if the rotation changes, since the mesh should follow along with that (if it was prevented above).
			// This should be rare that rotation changes during simulation, but it can happen when ShouldRemainVertical() changes, or standing on a moving base.
			const bool bValidateRotation = bPreventMeshMovement && (NetworkSmoothingMode == ENetworkSmoothingMode::Linear);
			if (bValidateRotation && UpdatedComponent)
			{
				// Same mesh with different rotation?
				const FQuat NewCapsuleRotation = UpdatedComponent->GetComponentQuat();
				if (Mesh == CharacterOwner->GetMesh() && !NewCapsuleRotation.Equals(SavedCapsuleRotation, 1e-6f) && ClientPredictionData)
				{
					// Add delta rotation to the target rotation and original offset. Otherwise this object will move back toward the old rotation.
					const FQuat RotationDelta = NewCapsuleRotation - SavedCapsuleRotation;
					ClientPredictionData->MeshRotationTarget += RotationDelta;
					ClientPredictionData->OriginalMeshRotationOffset += RotationDelta;

					// Update the MeshRotationOffset to match the capsule rotation.
					ClientPredictionData->MeshRotationOffset = NewCapsuleRotation;
					Mesh->SetRelativeLocationAndRotation(SavedMeshRelativeLocation, CharacterOwner->GetBaseRotationOffset());
				}
			}
		}

		if (bWasSimulatingRootMotion)
		{
			bWasSimulatingRootMotion = false;
		}
	}

	// Smooth mesh location after moving the capsule above.
	if (!bNetworkSmoothingComplete)
	{
		SCOPE_CYCLE_COUNTER(STAT_CharacterMovementSmoothClientPosition);
		SmoothClientPosition(DeltaSeconds);
	}
	else
	{
		UE_LOGF(LogCharacterNetSmoothing, Verbose, "Skipping network smoothing for %ls.", *GetNameSafe(CharacterOwner));
	}
}

FTransform UCharacterMovementComponent::ConvertLocalRootMotionToWorld(const FTransform& LocalRootMotionTransform, float DeltaSeconds)
{
	const FTransform PreProcessedRootMotion = ProcessRootMotionPreConvertToWorld.IsBound() ? ProcessRootMotionPreConvertToWorld.Execute(LocalRootMotionTransform, this, DeltaSeconds) : LocalRootMotionTransform;
	const FTransform WorldSpaceRootMotion = CharacterOwner->GetMesh()->ConvertLocalRootMotionToWorld(PreProcessedRootMotion);
	return ProcessRootMotionPostConvertToWorld.IsBound() ? ProcessRootMotionPostConvertToWorld.Execute(WorldSpaceRootMotion, this, DeltaSeconds) : WorldSpaceRootMotion;
}


void UCharacterMovementComponent::SimulateRootMotion(float DeltaSeconds, const FTransform& LocalRootMotionTransform)
{
	if( CharacterOwner && CharacterOwner->GetMesh() && (DeltaSeconds > 0.f) )
	{
		FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, bEnableScopedMovementUpdates);

		MaybeUpdateBasedMovement(DeltaSeconds);

		// Convert Local Space Root Motion to world space. Do it right before used by physics to make sure we use up to date transforms, as translation is relative to rotation.
		const FTransform WorldSpaceRootMotionTransform = ConvertLocalRootMotionToWorld(LocalRootMotionTransform, DeltaSeconds);
		RootMotionParams.Set( WorldSpaceRootMotionTransform );

		// Compute root motion velocity to be used by physics
		AnimRootMotionVelocity = CalcAnimRootMotionVelocity(WorldSpaceRootMotionTransform.GetTranslation(), DeltaSeconds, Velocity);
		Velocity = ConstrainAnimRootMotionVelocity(AnimRootMotionVelocity, Velocity);

		// Update replicated gravity direction
		if (bNetworkGravityDirectionChanged)
		{
			SetGravityDirection(CharacterOwner->GetReplicatedGravityDirection());
			bNetworkGravityDirectionChanged = false;
		}

		// Update replicated movement mode.
		if (bNetworkMovementModeChanged)
		{
			ApplyNetworkMovementMode(CharacterOwner->GetReplicatedMovementMode());
			bNetworkMovementModeChanged = false;
		}

		NumJumpApexAttempts = 0;
		StartNewPhysics(DeltaSeconds, 0);
		// fixme laurent - simulate movement seems to have step up issues? investigate as that would be cheaper to use.
		// 		SimulateMovement(DeltaSeconds);

		// Apply Root Motion rotation after movement is complete.
		const FQuat RootMotionRotationQuat = WorldSpaceRootMotionTransform.GetRotation();
		if( !RootMotionRotationQuat.IsIdentity() )
		{
			const FQuat NewActorRotationQuat = RootMotionRotationQuat * UpdatedComponent->GetComponentQuat();
			MoveUpdatedComponent(FVector::ZeroVector, NewActorRotationQuat, true);
		}

		MaybeSaveBaseLocation();
	}

	// Root Motion has been used, clear
	RootMotionParams.Clear();
}

FVector UCharacterMovementComponent::CalcAnimRootMotionVelocity(const FVector& RootMotionDeltaMove, float DeltaSeconds, const FVector& CurrentVelocity) const
{
	if (ensure(DeltaSeconds > 0.f))
	{
		FVector RootMotionVelocity = RootMotionDeltaMove / DeltaSeconds;
		return RootMotionVelocity;
	}
	else
	{
		return CurrentVelocity;
	}
}


FVector UCharacterMovementComponent::ConstrainAnimRootMotionVelocity(const FVector& RootMotionVelocity, const FVector& CurrentVelocity) const
{
	FVector Result = RootMotionVelocity;

	// Do not override Velocity.Z if in falling physics, we want to keep the effect of gravity.
	if (IsFalling())
	{
		if (HasCustomGravity())
		{
			SetGravitySpaceZ(Result, GetGravitySpaceZ(CurrentVelocity));
		}
		else
		{
			Result.Z = CurrentVelocity.Z;
		}
	}

	return Result;
}

void UCharacterMovementComponent::SimulateMovement(float DeltaSeconds)
{
	if (!HasValidData() || UpdatedComponent->Mobility != EComponentMobility::Movable || UpdatedComponent->IsSimulatingPhysics())
	{
		return;
	}

	const bool bIsSimulatedProxy = (CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy);

	const FRepMovement& ConstRepMovement = CharacterOwner->GetReplicatedMovement();

	// Workaround for replication not being updated initially
	if (bIsSimulatedProxy &&
		ConstRepMovement.Location.IsZero() &&
		ConstRepMovement.Rotation.IsZero() &&
		ConstRepMovement.LinearVelocity.IsZero())
	{
		return;
	}

	const FBasedMovementInfo& RepBasedMovement = CharacterOwner->GetReplicatedBasedMovement();
	FMovementBaseInterfaceData RepMovementBaseInterfaceData = MovementBaseUtility::GetMovementBaseDataFromPhysicsOwner(RepBasedMovement.MovementBaseInterfaceData.PhysicsObjectOwner.Get());
	
	// If base is not resolved on the client, we should not try to simulate at all
	if (!RepMovementBaseInterfaceData.IsValid() && CharacterOwner->GetReplicatedBasedMovement().bServerHasBaseComponent)
	{
		UE_LOGF(LogCharacterMovement, Verbose, "Base for simulated character '%ls' is not resolved on client, skipping SimulateMovement", *CharacterOwner->GetName());
		return;
	}

	FVector OldVelocity;
	FVector OldLocation;

	// Scoped updates can improve performance of multiple MoveComponent calls.
	{
		FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, bEnableScopedMovementUpdates);

		bool bHandledNetUpdate = false;
		if (bIsSimulatedProxy)
		{
			// Handle network changes
			if (bNetworkUpdateReceived)
			{
				bNetworkUpdateReceived = false;
				bHandledNetUpdate = true;
				UE_LOGF(LogCharacterMovement, Verbose, "Proxy %ls received net update", *CharacterOwner->GetName());
				if (bNetworkGravityDirectionChanged)
				{
					SetGravityDirection(CharacterOwner->GetReplicatedGravityDirection());
					bNetworkGravityDirectionChanged = false;
				}

				if (bNetworkMovementModeChanged)
				{
					ApplyNetworkMovementMode(CharacterOwner->GetReplicatedMovementMode());
					bNetworkMovementModeChanged = false;
				}
				else if (bJustTeleported || bForceNextFloorCheck)
				{
					// Make sure floor is current. We will continue using the replicated base, if there was one.
					bJustTeleported = false;
					UpdateFloorFromAdjustment();
				}
			}
			else if (bForceNextFloorCheck)
			{
				UpdateFloorFromAdjustment();
			}
		}

		UpdateCharacterStateBeforeMovement(DeltaSeconds);

		if (MovementMode != MOVE_None)
		{
			//TODO: Also ApplyAccumulatedForces()?
			HandlePendingLaunch();
		}
		ClearAccumulatedForces();

		if (MovementMode == MOVE_None)
		{
			return;
		}

		const bool bSimGravityDisabled = (bIsSimulatedProxy && CharacterOwner->bSimGravityDisabled);
		const bool bZeroReplicatedGroundVelocity = (bIsSimulatedProxy && IsMovingOnGround() && ConstRepMovement.LinearVelocity.IsZero());
		
		// bSimGravityDisabled means velocity was zero when replicated and we were stuck in something. Avoid external changes in velocity as well.
		// Being in ground movement with zero velocity, we cannot simulate proxy velocities safely because we might not get any further updates from the server.
		if (bSimGravityDisabled || bZeroReplicatedGroundVelocity)
		{
			Velocity = FVector::ZeroVector;
		}

		MaybeUpdateBasedMovement(DeltaSeconds);

		// simulated pawns predict location
		OldVelocity = Velocity;
		OldLocation = UpdatedComponent->GetComponentLocation();

		UpdateProxyAcceleration();

		// May only need to simulate forward on frames where we haven't just received a new position update.
		if (!bHandledNetUpdate || !bNetworkSkipProxyPredictionOnNetUpdate || !CharacterMovementCVars::NetEnableSkipProxyPredictionOnNetUpdate)
		{
			UE_LOGF(LogCharacterMovement, Verbose, "Proxy %ls simulating movement", *GetNameSafe(CharacterOwner));
			FStepDownResult StepDownResult;
			MoveSmooth(Velocity, DeltaSeconds, &StepDownResult);

			// find floor and check if falling
			if (IsMovingOnGround() || MovementMode == MOVE_Falling)
			{
				const bool bShouldFindFloor = GetGravitySpaceZ(Velocity) <= UE_KINDA_SMALL_NUMBER;

				if (StepDownResult.bComputedFloor)
				{
					CurrentFloor = StepDownResult.FloorResult;
				}
				else if (bShouldFindFloor)
				{
					FindFloor(UpdatedComponent->GetComponentLocation(), CurrentFloor, Velocity.IsZero(), NULL);
				}
				else
				{
					CurrentFloor.Clear();
				}

				// Possible for dynamic movement bases, particularly those that align to slopes while the character does not, to encroach the character.
				// Check to see if we can resolve the penetration in those cases, and if so find the floor.
				if (CurrentFloor.HitResult.bStartPenetrating && MovementBaseUtility::IsDynamicBase(GetMovementBaseInterfaceData()))
				{
					// Follows PhysWalking approach for encroachment on floor tests
					FHitResult Hit(CurrentFloor.HitResult);
					Hit.TraceEnd = Hit.TraceStart - GetGravityDirection() * MAX_FLOOR_DIST;
					
					const FVector RequestedAdjustment = GetPenetrationAdjustment(Hit);
					const bool bResolved = ResolvePenetration(RequestedAdjustment, Hit, UpdatedComponent->GetComponentQuat());
					bForceNextFloorCheck |= bResolved;
				}
				else if (!CurrentFloor.IsWalkableFloor())
				{
					if (!bSimGravityDisabled)
					{
						// No floor, must fall.
						if (GetGravitySpaceZ(Velocity) <= UE_KINDA_SMALL_NUMBER || bApplyGravityWhileJumping || !CharacterOwner->IsJumpProvidingForce())
						{
							Velocity = NewFallVelocity(Velocity, -GetGravityDirection() * GetGravityZ(), DeltaSeconds);
						}
					}
					SetMovementMode(MOVE_Falling);
				}
				else
				{
					// Walkable floor
					if (IsMovingOnGround())
					{
						AdjustFloorHeight();
						SetBaseFromFloor(CurrentFloor);
					}
					else if (MovementMode == MOVE_Falling)
					{
						if (CurrentFloor.FloorDist <= MIN_FLOOR_DIST || (bSimGravityDisabled && CurrentFloor.FloorDist <= MAX_FLOOR_DIST))
						{
							// Landed
							SetPostLandedPhysics(CurrentFloor.HitResult);
						}
						else
						{
							if (!bSimGravityDisabled)
							{
								// Continue falling.
								Velocity = NewFallVelocity(Velocity, -GetGravityDirection() * GetGravityZ(), DeltaSeconds);
							}
							CurrentFloor.Clear();
						}
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogCharacterMovement, Verbose, "Proxy %ls SKIPPING simulate movement", *GetNameSafe(CharacterOwner));
		}

		UpdateCharacterStateAfterMovement(DeltaSeconds);

		// consume path following requested velocity
		LastUpdateRequestedVelocity = bHasRequestedVelocity ? RequestedVelocity : FVector::ZeroVector;
		bHasRequestedVelocity = false;

		OnMovementUpdated(DeltaSeconds, OldLocation, OldVelocity);
	} // End scoped movement update

	// Call custom post-movement events. These happen after the scoped movement completes in case the events want to use the current state of overlaps etc.
	CallMovementUpdateDelegate(DeltaSeconds, OldLocation, OldVelocity);

	if (CharacterMovementCVars::BasedMovementMode == 0)
	{
		SaveBaseLocation(); // behaviour before implementing this fix
	}
	else
	{
		MaybeSaveBaseLocation();
	}
	UpdateComponentVelocity();
	bJustTeleported = false;

	LastUpdateLocation = UpdatedComponent ? UpdatedComponent->GetComponentLocation() : FVector::ZeroVector;
	LastUpdateRotation = UpdatedComponent ? UpdatedComponent->GetComponentQuat() : FQuat::Identity;
	LastUpdateVelocity = Velocity;
}

UPrimitiveComponent* UCharacterMovementComponent::GetMovementBase() const
{
	const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
	return MovementBaseInterfaceData ? Cast<UPrimitiveComponent>(MovementBaseInterfaceData->GetMovementBaseObject()) : nullptr;
}

UObject* UCharacterMovementComponent::GetMovementBaseObject() const
{
	const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
	return MovementBaseInterfaceData ? MovementBaseInterfaceData->GetMovementBaseObject() : nullptr;
}

const FMovementBaseInterfaceData* UCharacterMovementComponent::GetMovementBaseInterfaceData() const
{
	return CharacterOwner ? CharacterOwner->GetMovementBaseInterfaceData() : nullptr;
}

FMovementBaseInterfaceData* UCharacterMovementComponent::GetMovementBaseInterfaceData_Mutable()
{
	return CharacterOwner ? CharacterOwner->GetMovementBaseInterfaceData_Mutable() : nullptr;
}

void UCharacterMovementComponent::SetBase(FMovementBaseInterfaceData* MovementBaseInterfaceData, const FName BoneName, bool bNotifyActor)
{
	// prevent from changing Base while server is NavWalking (no Base in that mode), so both sides are in sync
	// otherwise it will cause problems with position smoothing

	if (CharacterOwner && !bIsNavWalkingOnServer)
	{
		if (bBaseOnAttachmentRoot && MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid() && BoneName == NAME_None)
		{
			// TODO: Support bBaseOnAttachmentRoot for UObjects
			if (UPrimitiveComponent* MovementBaseAsPrimitiveComponent = Cast<UPrimitiveComponent>(MovementBaseInterfaceData->PhysicsObjectOwner))
			{
				// The attachment root might not be a PrimitiveComponent so check first
				if (UPrimitiveComponent* NewRootBase = Cast<UPrimitiveComponent>(MovementBaseAsPrimitiveComponent->GetAttachmentRoot()))
				{
					FMovementBaseInterfaceData MovementBaseDataAsPrimitive(NewRootBase);
					MovementBaseInterfaceData->Set(MovementBaseDataAsPrimitive.PhysicsObjectOwner.Get(), MovementBaseDataAsPrimitive.GetBodyInstanceOwner());
				}
			}
		}
		
		CharacterOwner->SetBase(MovementBaseInterfaceData, MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid() ? BoneName : NAME_None, bNotifyActor);
	}
}

void UCharacterMovementComponent::SetBaseFromFloor(const FFindFloorResult& FloorResult)
{
	if (FloorResult.IsWalkableFloor())
	{
		const FFindFloorResult InitialFloor = FloorResult;

		FMovementBaseInterfaceData MovementBaseInterfaceData = MovementBaseUtility::GetMovementBaseDataFromHitResult(&FloorResult.HitResult);
		
		SetBase(&MovementBaseInterfaceData, FloorResult.HitResult.BoneName);

		// Support valid hit results that don't return a PrimitiveComponent.
		// SetBase will clear the floor in those cases, but we should keep the valid floor.
		// SetBase also never keeps the base when bIsNavWalkingOnServer, so don't bother keeping the floor there.
		if (CharacterMovementCVars::bRestoreFloorWhenMissingBaseComponent)
		{
			if (!MovementBaseUtility::IsMovementBaseDataValid(GetMovementBaseInterfaceData()) && !bIsNavWalkingOnServer)
			{
				CurrentFloor = InitialFloor;
			}
		}
	}
	else
	{
		SetBase(static_cast<FMovementBaseInterfaceData*>(nullptr));
	}
}

void UCharacterMovementComponent::MaybeUpdateBasedMovement(float DeltaSeconds)
{
	bDeferUpdateBasedMovement = false;

	FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData_Mutable();
	if (MovementBaseUtility::UseRelativeLocation(MovementBaseInterfaceData))
	{
		// Need to see if anything we're on is simulating physics or has a parent that is.		
		if (!MovementBaseUtility::IsSimulatedBase(MovementBaseInterfaceData))
		{
			bDeferUpdateBasedMovement = false;
			UpdateBasedMovement(DeltaSeconds);
			// If previously simulated, go back to using normal tick dependencies.
			if (PostPhysicsTickFunction.IsTickFunctionEnabled())
			{
				PostPhysicsTickFunction.SetTickFunctionEnable(false);
				MovementBaseUtility::AddTickDependency(PrimaryComponentTick, MovementBaseInterfaceData);
			}
		}
		else
		{
			// defer movement base update until after physics
			bDeferUpdateBasedMovement = true;
			// If previously not simulating, remove tick dependencies and use post physics tick function.
			if (!PostPhysicsTickFunction.IsTickFunctionEnabled())
			{
				PostPhysicsTickFunction.SetTickFunctionEnable(true);
				MovementBaseUtility::RemoveTickDependency(PrimaryComponentTick, MovementBaseInterfaceData);
			}

			if (CharacterMovementCVars::BasedMovementMode == 2)
			{
				UpdateBasedMovement(DeltaSeconds);
			}
		}
	}
	else
	{
		// Remove any previous physics tick dependencies. SetBase() takes care of the other dependencies.
		if (PostPhysicsTickFunction.IsTickFunctionEnabled())
		{
			PostPhysicsTickFunction.SetTickFunctionEnable(false);
		}
	}
}

void UCharacterMovementComponent::MaybeSaveBaseLocation()
{
	if (CharacterMovementCVars::BasedMovementMode == 2 || !bDeferUpdateBasedMovement)
	{
		SaveBaseLocation();
	}
}

// @todo handle lift moving up and down through encroachment
void UCharacterMovementComponent::UpdateBasedMovement(float DeltaSeconds)
{
	if (!HasValidData())
	{
		return;
	}

	const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
	if (!MovementBaseInterfaceData || !MovementBaseInterfaceData->IsValid() || !MovementBaseUtility::UseRelativeLocation(MovementBaseInterfaceData))
	{
		return;
	}

	if (!MovementBaseUtility::IsMovementBaseDataValid(MovementBaseInterfaceData))
	{
		FMovementBaseInterfaceData EmptyMovementBaseData;
		SetBase(&EmptyMovementBaseData);
		return;
	}

	// Ignore collision with bases during these movements.
	TGuardValue<EMoveComponentFlags> ScopedFlagRestore(MoveComponentFlags, MoveComponentFlags | MOVECOMP_IgnoreBases);

	FQuat DeltaQuat = FQuat::Identity;
	FVector DeltaPosition = FVector::ZeroVector;

	FQuat NewBaseQuat;
	FVector NewBaseLocation;
	if (!MovementBaseUtility::GetMovementBaseTransform(MovementBaseInterfaceData, CharacterOwner->GetBasedMovement().BoneName, NewBaseLocation, NewBaseQuat))
	{
		return;
	}

	// Find change in rotation
	const bool bRotationChanged = !OldBaseQuat.Equals(NewBaseQuat, 1e-8f);
	if (bRotationChanged)
	{
		DeltaQuat = NewBaseQuat * OldBaseQuat.Inverse();
	}

	// only if base moved
	if (bRotationChanged || (OldBaseLocation != NewBaseLocation))
	{
		// Calculate new transform matrix of base actor (ignoring scale).
		const FQuatRotationTranslationMatrix OldLocalToWorld(OldBaseQuat, OldBaseLocation);
		const FQuatRotationTranslationMatrix NewLocalToWorld(NewBaseQuat, NewBaseLocation);

		FQuat FinalQuat = UpdatedComponent->GetComponentQuat();
			
		if (bRotationChanged && !bIgnoreBaseRotation)
		{
			// Apply change in rotation and pipe through FaceRotation to maintain axis restrictions
			const FQuat PawnOldQuat = UpdatedComponent->GetComponentQuat();
			const FQuat TargetQuat = DeltaQuat * FinalQuat;
			FRotator TargetRotator(TargetQuat);
			CharacterOwner->FaceRotation(TargetRotator, 0.f);
			FinalQuat = UpdatedComponent->GetComponentQuat();

			if (PawnOldQuat.Equals(FinalQuat, 1e-6f))
			{
				// Nothing changed. This means we probably are using another rotation mechanism (bOrientToMovement etc). We should still follow the base object.
				// @todo: This assumes only Yaw is used, currently a valid assumption. This is the only reason FaceRotation() is used above really, aside from being a virtual hook.
				if (bOrientRotationToMovement || (bUseControllerDesiredRotation && CharacterOwner->GetController()))
				{
					// Custom gravity automatically aligns the character to the gravity direction, so we shouldn't zero out pitch and roll.
					if (!HasCustomGravity())
					{
						TargetRotator.Pitch = 0.f;
						TargetRotator.Roll = 0.f;
					}
					MoveUpdatedComponent(FVector::ZeroVector, TargetRotator, false);
					FinalQuat = UpdatedComponent->GetComponentQuat();
				}
			}

			// Pipe through ControlRotation, to affect camera.
			if (CharacterOwner->GetController())
			{
				const FQuat PawnDeltaRotation = FinalQuat * PawnOldQuat.Inverse();
				FRotator FinalRotation = FinalQuat.Rotator();
				UpdateBasedRotation(FinalRotation, PawnDeltaRotation.Rotator());
				FinalQuat = UpdatedComponent->GetComponentQuat();
			}
		}

		FVector NewWorldPos;
		if (HasCustomGravity())
		{
			const FVector RotationRadius = UpdatedComponent->GetComponentLocation() - NewBaseLocation;
			const FVector RotationDelta = DeltaQuat.RotateVector(RotationRadius) - RotationRadius;
			const FVector LinearDelta = NewBaseLocation - OldBaseLocation;
			NewWorldPos = ConstrainLocationToPlane(UpdatedComponent->GetComponentLocation() + RotationDelta + LinearDelta);
		}
		else
		{
			// We need to offset the base of the character here, not its origin, so offset by half height
			float HalfHeight, Radius;
			CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(Radius, HalfHeight);

			FVector const BaseOffset = FVector(0.0f, 0.0f, HalfHeight);
			FVector const LocalBasePos = OldLocalToWorld.InverseTransformPosition(UpdatedComponent->GetComponentLocation() - BaseOffset);
			NewWorldPos = ConstrainLocationToPlane(NewLocalToWorld.TransformPosition(LocalBasePos) + BaseOffset);
		}
		
		DeltaPosition = ConstrainDirectionToPlane(NewWorldPos - UpdatedComponent->GetComponentLocation());

		// move attached actor
		if (bFastAttachedMove)
		{
			// we're trusting no other obstacle can prevent the move here
			UpdatedComponent->SetWorldLocationAndRotation(NewWorldPos, FinalQuat, false);
		}
		else
		{
			// Set MovementBase's root actor as ignored when moving the character primitive component, 
			// only perform if bDeferUpdateBasedMovement is true since this means the MovementBase is simulating physics
			const bool bIgnoreBaseActor = bDeferUpdateBasedMovement && bBasedMovementIgnorePhysicsBase;
			AActor* MovementBaseRootActor = nullptr;
			if (bIgnoreBaseActor)
			{
				if (MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid())
				{
					MovementBaseRootActor = Cast<AActor>(MovementBaseInterfaceData->GetBodyInstanceOwner()->GetPhysicsOwnerAttachmentRoot());
				}
				UpdatedPrimitive->IgnoreActorWhenMoving(MovementBaseRootActor, true);
				MoveComponentFlags |= MOVECOMP_CheckBlockingRootActorInIgnoreList; // Hit actors during MoveUpdatedComponent will have their root actor compared with the ignored actors array 
			}

			// hack - transforms between local and world space introducing slight error FIXMESTEVE - discuss with engine team: just skip the transforms if no rotation?
			FVector BaseMoveDelta = NewBaseLocation - OldBaseLocation;
			if (!bRotationChanged && (BaseMoveDelta.X == 0.f) && (BaseMoveDelta.Y == 0.f))
			{
				DeltaPosition.X = 0.f;
				DeltaPosition.Y = 0.f;
			}

			FHitResult MoveOnBaseHit(1.f);
			const FVector OldLocation = UpdatedComponent->GetComponentLocation();
			MoveUpdatedComponent(DeltaPosition, FinalQuat, true, &MoveOnBaseHit);

			if ((UpdatedComponent->GetComponentLocation() - (OldLocation + DeltaPosition)).IsNearlyZero() == false)
			{
				OnUnableToFollowBaseMove(DeltaPosition, OldLocation, MoveOnBaseHit);
			}

			// Reset base actor ignore state
			if (bIgnoreBaseActor)
			{
				MoveComponentFlags &= ~MOVECOMP_CheckBlockingRootActorInIgnoreList;
				UpdatedPrimitive->IgnoreActorWhenMoving(MovementBaseRootActor, false);
			}
		}

		if (MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid())
		{
			const FBodyInstance* MovementBaseBodyInstance = MovementBaseInterfaceData->GetBodyInstanceOwner()->GetBodyInstance();
			if (MovementBaseBodyInstance && MovementBaseBodyInstance->IsInstanceSimulatingPhysics() && CharacterOwner->GetMesh())
			{
				CharacterOwner->GetMesh()->ApplyDeltaToAllPhysicsTransforms(DeltaPosition, DeltaQuat);
			}
		}
	}

	// Check if falling above current base
	if (IsFalling() && bStayBasedInAir)
	{
		const FVector PawnLocation = UpdatedComponent->GetComponentLocation();
		FFindFloorResult OutFloorResult;
		ComputeFloorDist(PawnLocation, StayBasedInAirHeight, StayBasedInAirHeight, OutFloorResult, CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleRadius(), NULL);

		FMovementBaseInterfaceData FoundMovementBaseData = MovementBaseUtility::GetMovementBaseDataFromHitResult(&OutFloorResult.HitResult);
		if (!FoundMovementBaseData.IsValid() || FoundMovementBaseData.GetBodyInstanceOwner()->GetPhysicsOwnerAttachmentRoot() != MovementBaseInterfaceData->GetBodyInstanceOwner()->GetPhysicsOwnerAttachmentRoot())
		{
			// New or no base under the character
			ApplyImpartedMovementBaseVelocity();
			FMovementBaseInterfaceData EmptyMovementBaseData;
			SetBase(&EmptyMovementBaseData);
			return;
		}
	}
}


void UCharacterMovementComponent::OnUnableToFollowBaseMove(const FVector& DeltaPosition, const FVector& OldLocation, const FHitResult& MoveOnBaseHit)
{
	// no default implementation, left for subclasses to override.
}


void UCharacterMovementComponent::UpdateBasedRotation(FRotator& FinalRotation, const FRotator& ReducedRotation)
{
	AController* Controller = CharacterOwner ? CharacterOwner->GetController() : nullptr;
	float ControllerRoll = 0.f;

	if ((Controller != nullptr) && !bIgnoreBaseRotation)
	{
		FRotator ControllerRot = Controller->GetControlRotation();
		ControllerRoll = ControllerRot.Roll;
		Controller->SetControlRotation(ControllerRot + ReducedRotation);
	}

	// Remove roll
	FinalRotation.Roll = 0.f;
	if (Controller != nullptr)
	{
		check(UpdatedComponent != nullptr);
		FinalRotation.Roll = UpdatedComponent->GetComponentRotation().Roll;
		FRotator NewRotation = Controller->GetControlRotation();
		NewRotation.Roll = ControllerRoll;
		Controller->SetControlRotation(NewRotation);
	}
}

void UCharacterMovementComponent::ApplyImpartedMovementBaseVelocity()
{
	DecayingFormerBaseVelocity = GetImpartedMovementBaseVelocity();
	Velocity += DecayingFormerBaseVelocity;
	if (bMovementInProgress && CurrentRootMotion.HasAdditiveVelocity())
	{
		// If we leave a base during movement and we have additive root motion, we need to add the imparted velocity so that it retains it next tick
		CurrentRootMotion.LastPreAdditiveVelocity += DecayingFormerBaseVelocity;
	}
	if (!CharacterMovementCVars::bAddFormerBaseVelocityToRootMotionOverrideWhenFalling || FormerBaseVelocityDecayHalfLife == 0.f)
	{
		DecayingFormerBaseVelocity = FVector::ZeroVector;
	}
}

void UCharacterMovementComponent::DisableMovement()
{
	if (CharacterOwner)
	{
		SetMovementMode(MOVE_None);
	}
	else
	{
		MovementMode = MOVE_None;
		CustomMovementMode = 0;
	}
}

void UCharacterMovementComponent::PerformMovement(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementPerformMovement);

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	// We will speculate during the movement and assume we only hit static geometry
	UE::RemoteExecutor::FSpeculationExecutionScope AssumeNoCrossServerHits;
#endif


	const UWorld* MyWorld = GetWorld();
	if (!HasValidData() || MyWorld == nullptr)
	{
		return;
	}

	bTeleportedSinceLastUpdate = UpdatedComponent->GetComponentLocation() != LastUpdateLocation;
	
	// no movement if we can't move, or if currently doing physical simulation on UpdatedComponent
	if (MovementMode == MOVE_None || UpdatedComponent->Mobility != EComponentMobility::Movable || UpdatedComponent->IsSimulatingPhysics())
	{
		if (!CharacterOwner->bClientUpdating && !CharacterOwner->bServerMoveIgnoreRootMotion)
		{
			// Consume root motion
			if (CharacterOwner->IsPlayingRootMotion() && CharacterOwner->GetMesh())
			{
				TickCharacterPose(DeltaSeconds);
				RootMotionParams.Clear();
			}
			if (CurrentRootMotion.HasActiveRootMotionSources())
			{
				CurrentRootMotion.Clear();
			}
		}
		// Clear pending physics forces
		ClearAccumulatedForces();
		return;
	}

	// Force floor update if we've moved outside of CharacterMovement since last update.
	bForceNextFloorCheck |= (IsMovingOnGround() && bTeleportedSinceLastUpdate);

	// Update saved LastPreAdditiveVelocity with any external changes to character Velocity that happened since last update.
	if( CurrentRootMotion.HasAdditiveVelocity() )
	{
		const FVector Adjustment = (Velocity - LastUpdateVelocity);
		CurrentRootMotion.LastPreAdditiveVelocity += Adjustment;

#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
		{
			if (!Adjustment.IsNearlyZero())
			{
				FString AdjustedDebugString = FString::Printf(TEXT("PerformMovement HasAdditiveVelocity LastUpdateVelocityAdjustment LastPreAdditiveVelocity(%s) Adjustment(%s)"),
					*CurrentRootMotion.LastPreAdditiveVelocity.ToCompactString(), *Adjustment.ToCompactString());
				RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
			}
		}
#endif
	}

	FVector OldVelocity;
	FVector OldLocation;

	// Scoped updates can improve performance of multiple MoveComponent calls.
	{
		FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, bEnableScopedMovementUpdates);

		MaybeUpdateBasedMovement(DeltaSeconds);

		// Clean up invalid RootMotion Sources.
		// This includes RootMotion sources that ended naturally.
		// They might want to perform a clamp on velocity or an override, 
		// so we want this to happen before ApplyAccumulatedForces and HandlePendingLaunch as to not clobber these.
		const bool bHasRootMotionSources = HasRootMotionSources();
		if (bHasRootMotionSources && !CharacterOwner->bClientUpdating && !CharacterOwner->bServerMoveIgnoreRootMotion)
		{
			SCOPE_CYCLE_COUNTER(STAT_CharacterMovementRootMotionSourceCalculate);

			const FVector VelocityBeforeCleanup = Velocity;
			CurrentRootMotion.CleanUpInvalidRootMotion(DeltaSeconds, *CharacterOwner, *this);

#if ROOT_MOTION_DEBUG
			if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
			{
				if (Velocity != VelocityBeforeCleanup)
				{
					const FVector Adjustment = Velocity - VelocityBeforeCleanup;
					FString AdjustedDebugString = FString::Printf(TEXT("PerformMovement CleanUpInvalidRootMotion Velocity(%s) VelocityBeforeCleanup(%s) Adjustment(%s)"),
						*Velocity.ToCompactString(), *VelocityBeforeCleanup.ToCompactString(), *Adjustment.ToCompactString());
					RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
				}
			}
#endif
		}

		OldVelocity = Velocity;
		OldLocation = UpdatedComponent->GetComponentLocation();

		ApplyAccumulatedForces(DeltaSeconds);

		// Update the character state before we do our movement
		UpdateCharacterStateBeforeMovement(DeltaSeconds);

		if (MovementMode == MOVE_NavWalking && bWantsToLeaveNavWalking)
		{
			TryToLeaveNavWalking();
		}

		// Character::LaunchCharacter() has been deferred until now.
		HandlePendingLaunch();
		ClearAccumulatedForces();

#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
		{
			if (OldVelocity != Velocity)
			{
				const FVector Adjustment = Velocity - OldVelocity;
				FString AdjustedDebugString = FString::Printf(TEXT("PerformMovement ApplyAccumulatedForces+HandlePendingLaunch Velocity(%s) OldVelocity(%s) Adjustment(%s)"),
					*Velocity.ToCompactString(), *OldVelocity.ToCompactString(), *Adjustment.ToCompactString());
				RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
			}
		}
#endif

		// Update saved LastPreAdditiveVelocity with any external changes to character Velocity that happened due to ApplyAccumulatedForces/HandlePendingLaunch
		if( CurrentRootMotion.HasAdditiveVelocity() )
		{
			const FVector Adjustment = (Velocity - OldVelocity);
			CurrentRootMotion.LastPreAdditiveVelocity += Adjustment;

#if ROOT_MOTION_DEBUG
			if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
			{
				if (!Adjustment.IsNearlyZero())
				{
					FString AdjustedDebugString = FString::Printf(TEXT("PerformMovement HasAdditiveVelocity AccumulatedForces LastPreAdditiveVelocity(%s) Adjustment(%s)"),
						*CurrentRootMotion.LastPreAdditiveVelocity.ToCompactString(), *Adjustment.ToCompactString());
					RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
				}
			}
#endif
		}

		// Prepare Root Motion (generate/accumulate from root motion sources to be used later)
		if (bHasRootMotionSources && !CharacterOwner->bClientUpdating && !CharacterOwner->bServerMoveIgnoreRootMotion)
		{
			// Animation root motion - If using animation RootMotion, tick animations before running physics.
			if( CharacterOwner->IsPlayingRootMotion() && CharacterOwner->GetMesh() )
			{
				TickCharacterPose(DeltaSeconds);

				// Make sure animation didn't trigger an event that destroyed us
				if (!HasValidData())
				{
					return;
				}

				// For local human clients, save off root motion data so it can be used by movement networking code.
				if( CharacterOwner->IsLocallyControlled() && (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy) && CharacterOwner->IsPlayingNetworkedRootMotionMontage() )
				{
					CharacterOwner->ClientRootMotionParams = RootMotionParams;
				}
			}

			// Generates root motion to be used this frame from sources other than animation
			{
				SCOPE_CYCLE_COUNTER(STAT_CharacterMovementRootMotionSourceCalculate);
				CurrentRootMotion.PrepareRootMotion(DeltaSeconds, *CharacterOwner, *this, true);
			}

			// For local human clients, save off root motion data so it can be used by movement networking code.
			if( CharacterOwner->IsLocallyControlled() && (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy) )
			{
				CharacterOwner->SavedRootMotion = CurrentRootMotion;
			}
		}

		// Apply Root Motion to Velocity
		if( CurrentRootMotion.HasOverrideVelocity() || HasAnimRootMotion() )
		{
			// Animation root motion overrides Velocity and currently doesn't allow any other root motion sources
			if( HasAnimRootMotion() )
			{
				// Convert to world space (animation root motion is always local)
				USkeletalMeshComponent * SkelMeshComp = CharacterOwner->GetMesh();
				if( SkelMeshComp )
				{
					// Convert Local Space Root Motion to world space. Do it right before used by physics to make sure we use up to date transforms, as translation is relative to rotation.
					RootMotionParams.Set( ConvertLocalRootMotionToWorld(RootMotionParams.GetRootMotionTransform(), DeltaSeconds) );
				}

				// Then turn root motion to velocity to be used by various physics modes.
				if( DeltaSeconds > 0.f )
				{
					AnimRootMotionVelocity = CalcAnimRootMotionVelocity(RootMotionParams.GetRootMotionTransform().GetTranslation(), DeltaSeconds, Velocity);
					Velocity = ConstrainAnimRootMotionVelocity(AnimRootMotionVelocity, Velocity);
					if (IsFalling())
					{
						Velocity += FVector(DecayingFormerBaseVelocity.X, DecayingFormerBaseVelocity.Y, 0.f);
					}
				}
				
				UE_LOGF(LogRootMotion, Log,  "PerformMovement WorldSpaceRootMotion Translation: %ls, Rotation: %ls, Actor Facing: %ls, Velocity: %ls"
					, *RootMotionParams.GetRootMotionTransform().GetTranslation().ToCompactString()
					, *RootMotionParams.GetRootMotionTransform().GetRotation().Rotator().ToCompactString()
					, *CharacterOwner->GetActorForwardVector().ToCompactString()
					, *Velocity.ToCompactString()
					);
			}
			else
			{
				// We don't have animation root motion so we apply other sources
				if( DeltaSeconds > 0.f )
				{
					SCOPE_CYCLE_COUNTER(STAT_CharacterMovementRootMotionSourceApply);

					const FVector VelocityBeforeOverride = Velocity;
					FVector NewVelocity = Velocity;
					CurrentRootMotion.AccumulateOverrideRootMotionVelocity(DeltaSeconds, *CharacterOwner, *this, NewVelocity);
					if (IsFalling())
					{
						NewVelocity += CurrentRootMotion.HasOverrideVelocityWithIgnoreZAccumulate() ? FVector(DecayingFormerBaseVelocity.X, DecayingFormerBaseVelocity.Y, 0.f) : DecayingFormerBaseVelocity;
					}
					Velocity = NewVelocity;

#if ROOT_MOTION_DEBUG
					if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
					{
						if (VelocityBeforeOverride != Velocity)
						{
							FString AdjustedDebugString = FString::Printf(TEXT("PerformMovement AccumulateOverrideRootMotionVelocity Velocity(%s) VelocityBeforeOverride(%s)"),
								*Velocity.ToCompactString(), *VelocityBeforeOverride.ToCompactString());
							RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
						}
					}
#endif
				}
			}
		}

#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
		{
			FString AdjustedDebugString = FString::Printf(TEXT("PerformMovement Velocity(%s) OldVelocity(%s)"),
				*Velocity.ToCompactString(), *OldVelocity.ToCompactString());
			RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
		}
#endif

		// NaN tracking
		devCode(ensureMsgf(!Velocity.ContainsNaN(), TEXT("UCharacterMovementComponent::PerformMovement: Velocity contains NaN (%s)\n%s"), *GetPathNameSafe(this), *Velocity.ToString()));

		// Clear jump input now, to allow movement events to trigger it for next update.
		CharacterOwner->ClearJumpInput(DeltaSeconds);
		NumJumpApexAttempts = 0;

		// Give one final chance to update the velocity right before doing the actual position change.
		UpdateVelocityBeforeMovement(DeltaSeconds);
		
		// change position
		StartNewPhysics(DeltaSeconds, 0);

		if (!HasValidData())
		{
			return;
		}

		// Update character state based on change from movement
		UpdateCharacterStateAfterMovement(DeltaSeconds);

		if (bAllowPhysicsRotationDuringAnimRootMotion || !HasAnimRootMotion())
		{
			PhysicsRotation(DeltaSeconds);
		}

		// Apply Root Motion rotation after movement is complete.
		if( HasAnimRootMotion() )
		{
			const FQuat OldActorRotationQuat = UpdatedComponent->GetComponentQuat();
			const FQuat RootMotionRotationQuat = RootMotionParams.GetRootMotionTransform().GetRotation();
			if( !RootMotionRotationQuat.IsIdentity() )
			{
				const FQuat NewActorRotationQuat = RootMotionRotationQuat * OldActorRotationQuat;
				MoveUpdatedComponent(FVector::ZeroVector, NewActorRotationQuat, true);
			}

#if !(UE_BUILD_SHIPPING)
			// debug
			#if 0
			{
				const FRotator OldActorRotation = OldActorRotationQuat.Rotator();
				const FVector ResultingLocation = UpdatedComponent->GetComponentLocation();
				const FRotator ResultingRotation = UpdatedComponent->GetComponentRotation();

				// Show current position
				DrawDebugCoordinateSystem(MyWorld, CharacterOwner->GetMesh()->GetComponentLocation() + FVector(0,0,1), ResultingRotation, 50.f, false);

				// Show resulting delta move.
				DrawDebugLine(MyWorld, OldLocation, ResultingLocation, FColor::Red, false, 10.f);

				const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
				// Log details.
				UE_LOGF(LogRootMotion, Warning,  "PerformMovement Resulting DeltaMove Translation: %ls, Rotation: %ls, MovementBase: %ls",
					*(ResultingLocation - OldLocation).ToCompactString(), *(ResultingRotation - OldActorRotation).GetNormalized().ToCompactString(), *GetNameSafe(MovementBaseInterfaceData ? MovementBaseInterfaceData->GetMovementBaseObject() : nullptr));

				const FVector RMTranslation = RootMotionParams.GetRootMotionTransform().GetTranslation();
				const FRotator RMRotation = RootMotionParams.GetRootMotionTransform().GetRotation().Rotator();
				UE_LOGF(LogRootMotion, Warning,  "PerformMovement Resulting DeltaError Translation: %ls, Rotation: %ls",
					*(ResultingLocation - OldLocation - RMTranslation).ToCompactString(), *(ResultingRotation - OldActorRotation - RMRotation).GetNormalized().ToCompactString() );
			}
			#endif
#endif // !(UE_BUILD_SHIPPING)

			// Root Motion has been used, clear
			RootMotionParams.Clear();
		}
		else if (CurrentRootMotion.HasActiveRootMotionSources())
		{
			FQuat RootMotionRotationQuat;
			if (CharacterOwner && UpdatedComponent && CurrentRootMotion.GetOverrideRootMotionRotation(DeltaSeconds, *CharacterOwner, *this, RootMotionRotationQuat))
			{
				const FQuat OldActorRotationQuat = UpdatedComponent->GetComponentQuat();
				const FQuat NewActorRotationQuat = RootMotionRotationQuat * OldActorRotationQuat;
				MoveUpdatedComponent(FVector::ZeroVector, NewActorRotationQuat, true);
			}
		}

		// consume path following requested velocity
		LastUpdateRequestedVelocity = bHasRequestedVelocity ? RequestedVelocity : FVector::ZeroVector;
		bHasRequestedVelocity = false;

		OnMovementUpdated(DeltaSeconds, OldLocation, OldVelocity);
	} // End scoped movement update

	// Call external post-movement events. These happen after the scoped movement completes in case the events want to use the current state of overlaps etc.
	CallMovementUpdateDelegate(DeltaSeconds, OldLocation, OldVelocity);

	if (CharacterMovementCVars::BasedMovementMode == 0)
	{
		SaveBaseLocation(); // behaviour before implementing this fix
	}
	else
	{
		MaybeSaveBaseLocation();
	}
	UpdateComponentVelocity();

	const bool bHasAuthority = CharacterOwner && CharacterOwner->HasAuthority();

	// If we move we want to avoid a long delay before replication catches up to notice this change, especially if it's throttling our rate.
	if (bHasAuthority && UNetDriver::IsAdaptiveNetUpdateFrequencyEnabled() && UpdatedComponent)
	{
		UNetDriver* NetDriver = MyWorld->GetNetDriver();
		if (NetDriver && NetDriver->IsServer())
		{
			if (!NetDriver->IsPendingNetUpdate(CharacterOwner) && NetDriver->IsNetworkActorUpdateFrequencyThrottled(CharacterOwner))
			{
				if (ShouldCancelAdaptiveReplication())
				{
					NetDriver->CancelAdaptiveReplication(CharacterOwner);
				}
			}
		}
	}

	const FVector NewLocation = UpdatedComponent ? UpdatedComponent->GetComponentLocation() : FVector::ZeroVector;
	const FQuat NewRotation = UpdatedComponent ? UpdatedComponent->GetComponentQuat() : FQuat::Identity;

	if (bHasAuthority && UpdatedComponent && !IsNetMode(NM_Client))
	{
		const bool bLocationChanged = (NewLocation != LastUpdateLocation);
		const bool bRotationChanged = (NewRotation != LastUpdateRotation);
		if (bLocationChanged || bRotationChanged)
		{
			// Update ServerLastTransformUpdateTimeStamp. This is used by Linear smoothing on clients to interpolate positions with the correct delta time,
			// so the timestamp should be based on the client's move delta (ServerAccumulatedClientTimeStamp), not the server time when receiving the RPC.
			const bool bIsRemotePlayer = (CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy);
			const FNetworkPredictionData_Server_Character* ServerData = bIsRemotePlayer ? GetPredictionData_Server_Character() : nullptr;
			if (bIsRemotePlayer && ServerData && CharacterMovementCVars::NetUseClientTimestampForReplicatedTransform)
			{
				ServerLastTransformUpdateTimeStamp = float(ServerData->ServerAccumulatedClientTimeStamp);
			}
			else
			{
				ServerLastTransformUpdateTimeStamp = MyWorld->GetTimeSeconds();
			}
		}
	}

	LastUpdateLocation = NewLocation;
	LastUpdateRotation = NewRotation;
	LastUpdateVelocity = Velocity;
}


bool UCharacterMovementComponent::ShouldCancelAdaptiveReplication() const
{
	// Update sooner if important properties changed.
	const bool bVelocityChanged = (Velocity != LastUpdateVelocity);
	const bool bLocationChanged = (UpdatedComponent->GetComponentLocation() != LastUpdateLocation);
	const bool bRotationChanged = (UpdatedComponent->GetComponentQuat() != LastUpdateRotation);

	return (bVelocityChanged || bLocationChanged || bRotationChanged);
}


void UCharacterMovementComponent::CallMovementUpdateDelegate(float DeltaTime, const FVector& OldLocation, const FVector& OldVelocity)
{
	SCOPE_CYCLE_COUNTER(STAT_CharMoveUpdateDelegate);

	// Update component velocity in case events want to read it
	UpdateComponentVelocity();

	// Delegate (for blueprints)
	if (CharacterOwner)
	{
		CharacterOwner->OnCharacterMovementUpdated.Broadcast(DeltaTime, OldLocation, OldVelocity);
	}
}


void UCharacterMovementComponent::OnMovementUpdated(float DeltaTime, const FVector& OldLocation, const FVector& OldVelocity)
{
	// empty base implementation, intended for derived classes to override.
}


void UCharacterMovementComponent::SaveBaseLocation()
{
	if (!HasValidData())
	{
		return;
	}

	const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
	if (MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid())
	{
		// Read transforms into OldBaseLocation, OldBaseQuat. Do this regardless of whether the object is movable, since mobility can change.
		MovementBaseUtility::GetMovementBaseTransform(MovementBaseInterfaceData, CharacterOwner->GetBasedMovement().BoneName, OldBaseLocation, OldBaseQuat);

		if (MovementBaseUtility::UseRelativeLocation(MovementBaseInterfaceData))
		{
			// Relative Location
			FVector RelativeLocation;
			MovementBaseUtility::TransformLocationToLocal(MovementBaseInterfaceData, CharacterOwner->GetBasedMovement().BoneName, UpdatedComponent->GetComponentLocation(), RelativeLocation);

			// Rotation
			if (bIgnoreBaseRotation)
			{
				// Absolute rotation
				CharacterOwner->SaveRelativeBasedMovement(RelativeLocation, UpdatedComponent->GetComponentRotation(), false);
			}
			else
			{
				// Relative rotation
				const FRotator RelativeRotation = (FQuatRotationMatrix(UpdatedComponent->GetComponentQuat()) * FQuatRotationMatrix(OldBaseQuat).GetTransposed()).Rotator();
				CharacterOwner->SaveRelativeBasedMovement(RelativeLocation, RelativeRotation, true);
			}
		}
	}
}


bool UCharacterMovementComponent::CanCrouchInCurrentState() const
{
	if (!CanEverCrouch())
	{
		return false;
	}

	return (IsFalling() || IsMovingOnGround()) && UpdatedComponent && !UpdatedComponent->IsSimulatingPhysics();
}

void UCharacterMovementComponent::SetCrouchedHalfHeight(const float NewValue)
{
	CrouchedHalfHeight = NewValue;

	if (CharacterOwner != nullptr)
	{
		CharacterOwner->RecalculateCrouchedEyeHeight();
	}
}

float UCharacterMovementComponent::GetCrouchedHalfHeight() const
{ 
	return CrouchedHalfHeight; 
}

void UCharacterMovementComponent::Crouch(bool bClientSimulation)
{
	if (!HasValidData())
	{
		return;
	}

	if (!bClientSimulation && !CanCrouchInCurrentState())
	{
		return;
	}

	// See if collision is already at desired size.
	if (CharacterOwner->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight() == CrouchedHalfHeight)
	{
		if (!bClientSimulation)
		{
			CharacterOwner->SetIsCrouched(true);
		}
		CharacterOwner->OnStartCrouch( 0.f, 0.f );
		return;
	}

	if (bClientSimulation && CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy)
	{
		// restore collision size before crouching
		ACharacter* DefaultCharacter = CharacterOwner->GetClass()->GetDefaultObject<ACharacter>();
		CharacterOwner->GetCapsuleComponent()->SetCapsuleSize(DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleRadius(), DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight());
		bShrinkProxyCapsule = true;
	}

	// Change collision size to crouching dimensions
	const float ComponentScale = CharacterOwner->GetCapsuleComponent()->GetShapeScale();
	const float OldUnscaledHalfHeight = CharacterOwner->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight();
	const float OldUnscaledRadius = CharacterOwner->GetCapsuleComponent()->GetUnscaledCapsuleRadius();
	// Height is not allowed to be smaller than radius.
	const float ClampedCrouchedHalfHeight = FMath::Max3(0.f, OldUnscaledRadius, CrouchedHalfHeight);
	CharacterOwner->GetCapsuleComponent()->SetCapsuleSize(OldUnscaledRadius, ClampedCrouchedHalfHeight);
	float HalfHeightAdjust = (OldUnscaledHalfHeight - ClampedCrouchedHalfHeight);
	float ScaledHalfHeightAdjust = HalfHeightAdjust * ComponentScale;

	if( !bClientSimulation )
	{
		// Crouching to a larger height? (this is rare)
		if (ClampedCrouchedHalfHeight > OldUnscaledHalfHeight)
		{
			FCollisionQueryParams CapsuleParams(SCENE_QUERY_STAT(CrouchTrace), false, CharacterOwner);
			FCollisionResponseParams ResponseParam;
			InitCollisionParams(CapsuleParams, ResponseParam);
			const bool bEncroached = GetWorld()->OverlapBlockingTestByChannel(UpdatedComponent->GetComponentLocation() + ScaledHalfHeightAdjust * GetGravityDirection(), GetWorldToGravityTransform(),
				UpdatedComponent->GetCollisionObjectType(), GetPawnCapsuleCollisionShape(SHRINK_None), CapsuleParams, ResponseParam);

			// If encroached, cancel
			if( bEncroached )
			{
				CharacterOwner->GetCapsuleComponent()->SetCapsuleSize(OldUnscaledRadius, OldUnscaledHalfHeight);
				return;
			}
		}

		if (bCrouchMaintainsBaseLocation)
		{
			// Intentionally not using MoveUpdatedComponent, where a horizontal plane constraint would prevent the base of the capsule from staying at the same spot.
			UpdatedComponent->MoveComponent(ScaledHalfHeightAdjust * GetGravityDirection(), UpdatedComponent->GetComponentQuat(), true, nullptr, EMoveComponentFlags::MOVECOMP_NoFlags, ETeleportType::TeleportPhysics);
		}

		CharacterOwner->SetIsCrouched(true);
	}

	bForceNextFloorCheck = true;

	// OnStartCrouch takes the change from the Default size, not the current one (though they are usually the same).
	const float MeshAdjust = ScaledHalfHeightAdjust;
	ACharacter* DefaultCharacter = CharacterOwner->GetClass()->GetDefaultObject<ACharacter>();
	HalfHeightAdjust = (DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight() - ClampedCrouchedHalfHeight);
	ScaledHalfHeightAdjust = HalfHeightAdjust * ComponentScale;

	AdjustProxyCapsuleSize();
	CharacterOwner->OnStartCrouch( HalfHeightAdjust, ScaledHalfHeightAdjust );

	// Don't smooth this change in mesh position
	if ((bClientSimulation && CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy) || (IsNetMode(NM_ListenServer) && CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy))
	{
		FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
		if (ClientData)
		{
			ClientData->MeshTranslationOffset -= MeshAdjust * -GetGravityDirection();
			ClientData->OriginalMeshTranslationOffset = ClientData->MeshTranslationOffset;
		}
	}
}


void UCharacterMovementComponent::UnCrouch(bool bClientSimulation)
{
	if (!HasValidData())
	{
		return;
	}

	ACharacter* DefaultCharacter = CharacterOwner->GetClass()->GetDefaultObject<ACharacter>();

	// See if collision is already at desired size.
	if( CharacterOwner->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight() == DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight() )
	{
		if (!bClientSimulation)
		{
			CharacterOwner->SetIsCrouched(false);
		}
		CharacterOwner->OnEndCrouch( 0.f, 0.f );
		return;
	}

	const float CurrentCrouchedHalfHeight = CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleHalfHeight();

	const float ComponentScale = CharacterOwner->GetCapsuleComponent()->GetShapeScale();
	const float OldUnscaledHalfHeight = CharacterOwner->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight();
	const float HalfHeightAdjust = DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight() - OldUnscaledHalfHeight;
	const float ScaledHalfHeightAdjust = HalfHeightAdjust * ComponentScale;
	const FVector PawnLocation = UpdatedComponent->GetComponentLocation();

	// Grow to uncrouched size.
	check(CharacterOwner->GetCapsuleComponent());

	if( !bClientSimulation )
	{
		// Try to stay in place and see if the larger capsule fits. We use a slightly taller capsule to avoid penetration.
		const UWorld* MyWorld = GetWorld();
		const float SweepInflation = UE_KINDA_SMALL_NUMBER * 10.f;
		FCollisionQueryParams CapsuleParams(SCENE_QUERY_STAT(CrouchTrace), false, CharacterOwner);
		FCollisionResponseParams ResponseParam;
		InitCollisionParams(CapsuleParams, ResponseParam);

		// Compensate for the difference between current capsule size and standing size
		const FCollisionShape StandingCapsuleShape = GetPawnCapsuleCollisionShape(SHRINK_HeightCustom, -SweepInflation - ScaledHalfHeightAdjust); // Shrink by negative amount, so actually grow it.
		const ECollisionChannel CollisionChannel = UpdatedComponent->GetCollisionObjectType();
		bool bEncroached = true;

		if (!bCrouchMaintainsBaseLocation)
		{
			// Expand in place
			bEncroached = MyWorld->OverlapBlockingTestByChannel(PawnLocation, GetWorldToGravityTransform(), CollisionChannel, StandingCapsuleShape, CapsuleParams, ResponseParam);
		
			if (bEncroached)
			{
				// Try adjusting capsule position to see if we can avoid encroachment.
				if (ScaledHalfHeightAdjust > 0.f)
				{
					// Shrink to a short capsule, sweep down to base to find where that would hit something, and then try to stand up from there.
					float PawnRadius, PawnHalfHeight;
					CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);
					const float ShrinkHalfHeight = PawnHalfHeight - PawnRadius;
					const float TraceDist = PawnHalfHeight - ShrinkHalfHeight;
					const FVector Down = TraceDist * GetGravityDirection();

					FHitResult Hit(1.f);
					const FCollisionShape ShortCapsuleShape = GetPawnCapsuleCollisionShape(SHRINK_HeightCustom, ShrinkHalfHeight);
					const bool bBlockingHit = MyWorld->SweepSingleByChannel(Hit, PawnLocation, PawnLocation + Down, GetWorldToGravityTransform(), CollisionChannel, ShortCapsuleShape, CapsuleParams);
					if (Hit.bStartPenetrating)
					{
						bEncroached = true;
					}
					else
					{
						// Compute where the base of the sweep ended up, and see if we can stand there
						const float DistanceToBase = (Hit.Time * TraceDist) + ShortCapsuleShape.Capsule.HalfHeight;
						const FVector Adjustment = (-DistanceToBase + StandingCapsuleShape.Capsule.HalfHeight + SweepInflation + MIN_FLOOR_DIST / 2.f) * -GetGravityDirection();
						const FVector NewLoc = PawnLocation + Adjustment;
						bEncroached = MyWorld->OverlapBlockingTestByChannel(NewLoc, GetWorldToGravityTransform(), CollisionChannel, StandingCapsuleShape, CapsuleParams, ResponseParam);
						if (!bEncroached)
						{
							// Intentionally not using MoveUpdatedComponent, where a horizontal plane constraint would prevent the base of the capsule from staying at the same spot.
							UpdatedComponent->MoveComponent(NewLoc - PawnLocation, UpdatedComponent->GetComponentQuat(), false, nullptr, EMoveComponentFlags::MOVECOMP_NoFlags, ETeleportType::TeleportPhysics);
						}
					}
				}
			}
		}
		else
		{
			// Expand while keeping base location the same.
			FVector StandingLocation = PawnLocation + (StandingCapsuleShape.GetCapsuleHalfHeight() - CurrentCrouchedHalfHeight) * -GetGravityDirection();
			bEncroached = MyWorld->OverlapBlockingTestByChannel(StandingLocation, GetWorldToGravityTransform(), CollisionChannel, StandingCapsuleShape, CapsuleParams, ResponseParam);

			if (bEncroached)
			{
				if (IsMovingOnGround())
				{
					// Something might be just barely overhead, try moving down closer to the floor to avoid it.
					const float MinFloorDist = UE_KINDA_SMALL_NUMBER * 10.f;
					if (CurrentFloor.bBlockingHit && CurrentFloor.FloorDist > MinFloorDist)
					{
						StandingLocation -= (CurrentFloor.FloorDist - MinFloorDist) * -GetGravityDirection();
						bEncroached = MyWorld->OverlapBlockingTestByChannel(StandingLocation, GetWorldToGravityTransform(), CollisionChannel, StandingCapsuleShape, CapsuleParams, ResponseParam);
					}
				}				
			}

			if (!bEncroached)
			{
				// Commit the change in location.
				UpdatedComponent->MoveComponent(StandingLocation - PawnLocation, UpdatedComponent->GetComponentQuat(), false, nullptr, EMoveComponentFlags::MOVECOMP_NoFlags, ETeleportType::TeleportPhysics);
				bForceNextFloorCheck = true;
			}
		}

		// If still encroached then abort.
		if (bEncroached)
		{
			return;
		}

		CharacterOwner->SetIsCrouched(false);
	}	
	else
	{
		bShrinkProxyCapsule = true;
	}

	// Now call SetCapsuleSize() to cause touch/untouch events and actually grow the capsule
	CharacterOwner->GetCapsuleComponent()->SetCapsuleSize(DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleRadius(), DefaultCharacter->GetCapsuleComponent()->GetUnscaledCapsuleHalfHeight(), true);

	const float MeshAdjust = ScaledHalfHeightAdjust;
	AdjustProxyCapsuleSize();
	CharacterOwner->OnEndCrouch( HalfHeightAdjust, ScaledHalfHeightAdjust );

	// Don't smooth this change in mesh position
	if ((bClientSimulation && CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy) || (IsNetMode(NM_ListenServer) && CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy))
	{
		FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
		if (ClientData)
		{
			ClientData->MeshTranslationOffset += MeshAdjust * -GetGravityDirection();
			ClientData->OriginalMeshTranslationOffset = ClientData->MeshTranslationOffset;
		}
	}
}

void UCharacterMovementComponent::UpdateCharacterStateBeforeMovement(float DeltaSeconds)
{
	// Proxies get replicated crouch state.
	if (CharacterOwner->GetLocalRole() != ROLE_SimulatedProxy)
	{
		// Check for a change in crouch state. Players toggle crouch by changing bWantsToCrouch.
		const bool bIsCrouching = IsCrouching();
		if (bIsCrouching && (!bWantsToCrouch || !CanCrouchInCurrentState()))
		{
			UnCrouch(false);
		}
		else if (!bIsCrouching && bWantsToCrouch && CanCrouchInCurrentState())
		{
			Crouch(false);
		}
	}
}

void UCharacterMovementComponent::UpdateCharacterStateAfterMovement(float DeltaSeconds)
{
	// Proxies get replicated crouch state.
	if (CharacterOwner->GetLocalRole() != ROLE_SimulatedProxy)
	{
		// Uncrouch if no longer allowed to be crouched
		if (IsCrouching() && !CanCrouchInCurrentState())
		{
			UnCrouch(false);
		}
	}
}

void UCharacterMovementComponent::StartNewPhysics(float deltaTime, int32 Iterations)
{
	if ((deltaTime < MIN_TICK_TIME) || (Iterations >= MaxSimulationIterations) || !HasValidData())
	{
		return;
	}

	if (UpdatedComponent->IsSimulatingPhysics())
	{
		UE_LOGF(LogCharacterMovement, Log, "UCharacterMovementComponent::StartNewPhysics: UpdateComponent (%ls) is simulating physics - aborting.", *UpdatedComponent->GetPathName());
		return;
	}

	const bool bSavedMovementInProgress = bMovementInProgress;
	bMovementInProgress = true;

	switch ( MovementMode )
	{
	case MOVE_None:
		break;
	case MOVE_Walking:
		PhysWalking(deltaTime, Iterations);
		break;
	case MOVE_NavWalking:
		PhysNavWalking(deltaTime, Iterations);
		break;
	case MOVE_Falling:
		PhysFalling(deltaTime, Iterations);
		break;
	case MOVE_Flying:
		PhysFlying(deltaTime, Iterations);
		break;
	case MOVE_Swimming:
		PhysSwimming(deltaTime, Iterations);
		break;
	case MOVE_Custom:
		PhysCustom(deltaTime, Iterations);
		break;
	default:
		UE_LOGF(LogCharacterMovement, Warning, "%ls has unsupported movement mode %d", *CharacterOwner->GetName(), int32(MovementMode));
		SetMovementMode(MOVE_None);
		break;
	}

	bMovementInProgress = bSavedMovementInProgress;
	if ( bDeferUpdateMoveComponent )
	{
		SetUpdatedComponent(DeferredUpdatedMoveComponent);
	}
}

float UCharacterMovementComponent::GetGravityZ() const
{
	return Super::GetGravityZ() * GravityScale;
}

float UCharacterMovementComponent::GetMaxSpeed() const
{
	switch(MovementMode)
	{
	case MOVE_Walking:
	case MOVE_NavWalking:
		return IsCrouching() ? MaxWalkSpeedCrouched : MaxWalkSpeed;
	case MOVE_Falling:
		return MaxWalkSpeed;
	case MOVE_Swimming:
		return MaxSwimSpeed;
	case MOVE_Flying:
		return MaxFlySpeed;
	case MOVE_Custom:
		return MaxCustomMovementSpeed;
	case MOVE_None:
	default:
		return 0.f;
	}
}

float UCharacterMovementComponent::GetMinAnalogSpeed() const
{
	switch (MovementMode)
	{
	case MOVE_Walking:
	case MOVE_NavWalking:
	case MOVE_Falling:
		return MinAnalogWalkSpeed;
	default:
		return 0.f;
	}
}

FVector UCharacterMovementComponent::GetPenetrationAdjustment(const FHitResult& Hit) const
{
	FVector Result = Super::GetPenetrationAdjustment(Hit);

	if (CharacterOwner)
	{
		const bool bIsProxy = (CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy);
		float MaxDistance = bIsProxy ? MaxDepenetrationWithGeometryAsProxy : MaxDepenetrationWithGeometry;
		if (Hit.HitObjectHandle.DoesRepresentClass(APawn::StaticClass()))
		{
			MaxDistance = bIsProxy ? MaxDepenetrationWithPawnAsProxy : MaxDepenetrationWithPawn;
		}

		Result = Result.GetClampedToMaxSize(MaxDistance);
	}

	return Result;
}


bool UCharacterMovementComponent::ResolvePenetrationImpl(const FVector& Adjustment, const FHitResult& Hit, const FQuat& NewRotation)
{
	// If movement occurs, mark that we teleported, so we don't incorrectly adjust velocity based on a potentially very different movement than our movement direction.
	bJustTeleported |= Super::ResolvePenetrationImpl(Adjustment, Hit, NewRotation);
	return bJustTeleported;
}


float UCharacterMovementComponent::SlideAlongSurface(const FVector& Delta, float Time, const FVector& InNormal, FHitResult& Hit, bool bHandleImpact)
{
	if (!Hit.bBlockingHit)
	{
		return 0.f;
	}

	FVector Normal(InNormal);
	const FVector::FReal NormalZ = GetGravitySpaceZ(Normal);
	if (IsMovingOnGround())
	{
		// We don't want to be pushed up an unwalkable surface.
		if (NormalZ > 0.f)
		{
			if (!IsWalkable(Hit))
			{
				Normal = ProjectToGravityFloor(Normal).GetSafeNormal();
			}
		}
		else if (NormalZ < -UE_KINDA_SMALL_NUMBER)
		{
			// Don't push down into the floor when the impact is on the upper portion of the capsule.
			if (CurrentFloor.FloorDist < MIN_FLOOR_DIST && CurrentFloor.bBlockingHit)
			{
				const FVector FloorNormal = CurrentFloor.HitResult.Normal;
				const bool bFloorOpposedToMovement = (Delta | FloorNormal) < 0.f && (GetGravitySpaceZ(FloorNormal) < 1.f - UE_DELTA);
				if (bFloorOpposedToMovement)
				{
					Normal = FloorNormal;
				}
				
				Normal = ProjectToGravityFloor(Normal).GetSafeNormal();
			}
		}
	}

	return Super::SlideAlongSurface(Delta, Time, Normal, Hit, bHandleImpact);
}


void UCharacterMovementComponent::TwoWallAdjust(FVector& WorldSpaceDelta, const FHitResult& Hit, const FVector& OldHitNormal) const
{
	const FVector InDelta = WorldSpaceDelta;
	Super::TwoWallAdjust(WorldSpaceDelta, Hit, OldHitNormal);

	if (IsMovingOnGround())
	{
		// Allow slides up walkable surfaces, but not unwalkable ones (treat those as vertical barriers).
		const FVector::FReal WorldSpaceDeltaZ = GetGravitySpaceZ(WorldSpaceDelta);
		if (WorldSpaceDeltaZ > 0.f)
		{
			const FVector::FReal HitNormalZ = GetGravitySpaceZ(Hit.Normal);
			if ((HitNormalZ >= WalkableFloorZ || IsWalkable(Hit)) && HitNormalZ > UE_KINDA_SMALL_NUMBER)
			{
				// Maintain horizontal velocity
				const float Time = (1.f - Hit.Time);
				const FVector ScaledDelta = WorldSpaceDelta.GetSafeNormal() * InDelta.Size();
				const FVector::FReal NewDeltaZ = (GetGravitySpaceZ(ScaledDelta) / HitNormalZ);
				WorldSpaceDelta = (ProjectToGravityFloor(InDelta) + NewDeltaZ * -GetGravityDirection()) * Time;

				// Should never exceed MaxStepHeight in vertical component, so rescale if necessary.
				// This should be rare (Hit.Normal.Z above would have been very small) but we'd rather lose horizontal velocity than go too high.
				const FVector::FReal DeltaZ = GetGravitySpaceZ(WorldSpaceDelta);
				if (DeltaZ > MaxStepHeight)
				{
					const float Rescale = MaxStepHeight / DeltaZ;
					WorldSpaceDelta *= Rescale;
				}
			}
			else
			{
				WorldSpaceDelta = ProjectToGravityFloor(WorldSpaceDelta);
			}
		}
		else if (WorldSpaceDeltaZ < 0.f)
		{
			// Don't push down into the floor.
			if (CurrentFloor.FloorDist < MIN_FLOOR_DIST && CurrentFloor.bBlockingHit)
			{
				WorldSpaceDelta = ProjectToGravityFloor(WorldSpaceDelta);
			}
		}
	}
}


FVector UCharacterMovementComponent::ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit) const
{
	FVector Result = Super::ComputeSlideVector(Delta, Time, Normal, Hit);

	// prevent boosting up slopes
	if (IsFalling())
	{
		Result = HandleSlopeBoosting(Result, Delta, Time, Normal, Hit);
	}

	return Result;
}


FVector UCharacterMovementComponent::HandleSlopeBoosting(const FVector& SlideResult, const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit) const
{
	FVector Result = SlideResult;
	const float ResultZ = GetGravitySpaceZ(Result);
	if (ResultZ > 0.f)
	{
		// Don't move any higher than we originally intended.
		const float ZLimit = GetGravitySpaceZ(Delta) * Time;
		if (ResultZ - ZLimit > UE_KINDA_SMALL_NUMBER)
		{
			if (ZLimit > 0.f)
			{
				// Rescale the entire vector (not just the Z component) otherwise we change the direction and likely head right back into the impact.
				const float UpPercent = ZLimit / ResultZ;
				Result *= UpPercent;
			}
			else
			{
				// We were heading down but were going to deflect upwards. Just make the deflection horizontal.
				Result = FVector::ZeroVector;
			}

			// Make remaining portion of original result horizontal and parallel to impact normal.
			const FVector RemainderXY = ProjectToGravityFloor(SlideResult - Result);
			const FVector NormalXY = ProjectToGravityFloor(Normal).GetSafeNormal();
			const FVector Adjust = Super::ComputeSlideVector(RemainderXY, 1.f, NormalXY, Hit);
			Result += Adjust;
		}
	}

	return Result;
}

FVector UCharacterMovementComponent::NewFallVelocity(const FVector& InitialVelocity, const FVector& Gravity, float DeltaTime) const
{
	FVector Result = InitialVelocity;

	if (DeltaTime > 0.f)
	{
		// Apply gravity.
		Result += Gravity * DeltaTime;

		// Don't exceed terminal velocity.
		const float TerminalLimit = FMath::Abs(GetPhysicsVolume()->TerminalVelocity);
		if (Result.SizeSquared() > FMath::Square(TerminalLimit))
		{
			const FVector GravityDir = Gravity.GetSafeNormal();
			if ((Result | GravityDir) > TerminalLimit)
			{
				Result = FVector::PointPlaneProject(Result, FVector::ZeroVector, GravityDir) + GravityDir * TerminalLimit;
			}
		}
	}

	return Result;
}


float UCharacterMovementComponent::ImmersionDepth() const
{
	float depth = 0.f;

	if ( CharacterOwner && GetPhysicsVolume()->bWaterVolume )
	{
		const float CollisionHalfHeight = CharacterOwner->GetSimpleCollisionHalfHeight();

		if ( (CollisionHalfHeight == 0.f) || (Buoyancy == 0.f) )
		{
			depth = 1.f;
		}
		else
		{
			UBrushComponent* VolumeBrushComp = GetPhysicsVolume()->GetBrushComponent();
			FHitResult Hit(1.f);
			if ( VolumeBrushComp )
			{
				const FVector TraceStart = UpdatedComponent->GetComponentLocation() + CollisionHalfHeight * -GetGravityDirection();
				const FVector TraceEnd = UpdatedComponent->GetComponentLocation() - CollisionHalfHeight * -GetGravityDirection();

				FCollisionQueryParams NewTraceParams(SCENE_QUERY_STAT(ImmersionDepth), true);
				VolumeBrushComp->LineTraceComponent( Hit, TraceStart, TraceEnd, NewTraceParams );
			}

			depth = (Hit.Time == 1.f) ? 1.f : (1.f - Hit.Time);
		}
	}
	return depth;
}

bool UCharacterMovementComponent::IsFlying() const
{
	return (MovementMode == MOVE_Flying) && UpdatedComponent;
}

bool UCharacterMovementComponent::IsMovingOnGround() const
{
	return ((MovementMode == MOVE_Walking) || (MovementMode == MOVE_NavWalking)) && UpdatedComponent;
}

bool UCharacterMovementComponent::IsFalling() const
{
	return (MovementMode == MOVE_Falling) && UpdatedComponent;
}

bool UCharacterMovementComponent::IsSwimming() const
{
	return (MovementMode == MOVE_Swimming) && UpdatedComponent;
}

bool UCharacterMovementComponent::IsCrouching() const
{
	return CharacterOwner && CharacterOwner->IsCrouched();
}

void UCharacterMovementComponent::CalcVelocity(float DeltaTime, float Friction, bool bFluid, float BrakingDeceleration)
{
	// Do not update velocity when using root motion or when SimulatedProxy and not simulating root motion - SimulatedProxy are repped their Velocity
	if (!HasValidData() || HasAnimRootMotion() || DeltaTime < MIN_TICK_TIME || (CharacterOwner && CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy && !bWasSimulatingRootMotion))
	{
		return;
	}

	Friction = FMath::Max(0.f, Friction);
	const float MaxAccel = GetMaxAcceleration();
	float MaxSpeed = GetMaxSpeed();
	
	// Check if path following requested movement
	bool bZeroRequestedAcceleration = true;
	FVector RequestedAcceleration = FVector::ZeroVector;
	float RequestedSpeed = 0.0f;
	if (ApplyRequestedMove(DeltaTime, MaxAccel, MaxSpeed, Friction, BrakingDeceleration, RequestedAcceleration, RequestedSpeed))
	{
		bZeroRequestedAcceleration = false;
	}

	if (bForceMaxAccel)
	{
		// Force acceleration at full speed.
		// In consideration order for direction: Acceleration, then Velocity, then Pawn's rotation.
		if (Acceleration.SizeSquared() > UE_SMALL_NUMBER)
		{
			Acceleration = Acceleration.GetSafeNormal() * MaxAccel;
		}
		else 
		{
			Acceleration = MaxAccel * (Velocity.SizeSquared() < UE_SMALL_NUMBER ? UpdatedComponent->GetForwardVector() : Velocity.GetSafeNormal());
		}

		AnalogInputModifier = 1.f;
	}

	// Path following above didn't care about the analog modifier, but we do for everything else below, so get the fully modified value.
	// Use max of requested speed and max speed if we modified the speed in ApplyRequestedMove above.
	const float MaxInputSpeed = FMath::Max(MaxSpeed * AnalogInputModifier, GetMinAnalogSpeed());
	MaxSpeed = FMath::Max(RequestedSpeed, MaxInputSpeed);

	// Apply braking or deceleration
	const bool bZeroAcceleration = Acceleration.IsZero();
	const bool bVelocityOverMax = IsExceedingMaxSpeed(MaxSpeed);
	
	// Only apply braking if there is no acceleration, or we are over our max speed and need to slow down to it.
	if ((bZeroAcceleration && bZeroRequestedAcceleration) || bVelocityOverMax)
	{
		const FVector OldVelocity = Velocity;

		const float ActualBrakingFriction = (bUseSeparateBrakingFriction ? BrakingFriction : Friction);
		ApplyVelocityBraking(DeltaTime, ActualBrakingFriction, BrakingDeceleration);
	
		// Don't allow braking to lower us below max speed if we started above it.
		if (bVelocityOverMax && Velocity.SizeSquared() < FMath::Square(MaxSpeed) && FVector::DotProduct(Acceleration, OldVelocity) > 0.0f)
		{
			Velocity = OldVelocity.GetSafeNormal() * MaxSpeed;
		}
	}
	else if (!bZeroAcceleration)
	{
		// Friction affects our ability to change direction. This is only done for input acceleration, not path following.
		const FVector AccelDir = Acceleration.GetSafeNormal();
		const float VelSize = Velocity.Size();
		Velocity = Velocity - (Velocity - AccelDir * VelSize) * FMath::Min(DeltaTime * Friction, 1.f);
	}

	// Apply fluid friction
	if (bFluid)
	{
		Velocity = Velocity * (1.f - FMath::Min(Friction * DeltaTime, 1.f));
	}

	// Apply input acceleration
	if (!bZeroAcceleration)
	{
		const float NewMaxInputSpeed = IsExceedingMaxSpeed(MaxInputSpeed) ? Velocity.Size() : MaxInputSpeed;
		Velocity += Acceleration * DeltaTime;
		Velocity = Velocity.GetClampedToMaxSize(NewMaxInputSpeed);
	}

	// Apply additional requested acceleration
	if (!bZeroRequestedAcceleration)
	{
		const float NewMaxRequestedSpeed = IsExceedingMaxSpeed(RequestedSpeed) ? Velocity.Size() : RequestedSpeed;
		Velocity += RequestedAcceleration * DeltaTime;
		Velocity = Velocity.GetClampedToMaxSize(NewMaxRequestedSpeed);
	}

	if (bUseRVOAvoidance)
	{
		CalcAvoidanceVelocity(DeltaTime);
	}
}

bool UCharacterMovementComponent::ShouldComputeAccelerationToReachRequestedVelocity(const float RequestedSpeed) const
{
	// Compute acceleration if accelerating toward requested speed, 1% buffer.
	return bRequestedMoveUseAcceleration && Velocity.SizeSquared() < FMath::Square(RequestedSpeed * 1.01f);
}

bool UCharacterMovementComponent::ApplyRequestedMove(float DeltaTime, float MaxAccel, float MaxSpeed, float Friction, float BrakingDeceleration, FVector& OutAcceleration, float& OutRequestedSpeed)
{
	if (bHasRequestedVelocity)
	{
		const float RequestedSpeedSquared = RequestedVelocity.SizeSquared();
		if (RequestedSpeedSquared < UE_KINDA_SMALL_NUMBER)
		{
			return false;
		}

		// Compute requested speed from path following
		float RequestedSpeed = FMath::Sqrt(RequestedSpeedSquared);
		const FVector RequestedMoveDir = RequestedVelocity / RequestedSpeed;
		RequestedSpeed = (bRequestedMoveWithMaxSpeed ? MaxSpeed : FMath::Min(MaxSpeed, RequestedSpeed));
		
		// Compute actual requested velocity
		const FVector MoveVelocity = RequestedMoveDir * RequestedSpeed;
		
		// Compute acceleration. Use MaxAccel to limit speed increase, 1% buffer.
		FVector NewAcceleration = FVector::ZeroVector;
		const float CurrentSpeedSq = Velocity.SizeSquared();
		if (ShouldComputeAccelerationToReachRequestedVelocity(RequestedSpeed))
		{
			// Turn in the same manner as with input acceleration.
			const float VelSize = FMath::Sqrt(CurrentSpeedSq);
			Velocity = Velocity - (Velocity - RequestedMoveDir * VelSize) * FMath::Min(DeltaTime * Friction, 1.f);

			// How much do we need to accelerate to get to the new velocity?
			NewAcceleration = ((MoveVelocity - Velocity) / DeltaTime);
			NewAcceleration = NewAcceleration.GetClampedToMaxSize(MaxAccel);
		}
		else
		{
			// Just set velocity directly.
			// If decelerating we do so instantly, so we don't slide through the destination if we can't brake fast enough.
			Velocity = MoveVelocity;
		}

		// Copy to out params
		OutRequestedSpeed = RequestedSpeed;
		OutAcceleration = NewAcceleration;
		return true;
	}

	return false;
}

FVector UCharacterMovementComponent::GetActorFeetLocation() const
{
	const UCapsuleComponent* const CapsuleComponent = CharacterOwner ? CharacterOwner->GetCapsuleComponent() : Cast<UCapsuleComponent>(UpdatedComponent);
	if (CapsuleComponent)
	{
		const float HalfHeight = CapsuleComponent->GetScaledCapsuleHalfHeight();
		return UpdatedComponent->GetComponentLocation() + HalfHeight * GetGravityDirection();
	}

	return Super::GetActorFeetLocation();
}

void UCharacterMovementComponent::RequestDirectMove(const FVector& MoveVelocity, bool bForceMaxSpeed)
{
	if (MoveVelocity.SizeSquared() < UE_KINDA_SMALL_NUMBER)
	{
		return;
	}

	if (ShouldPerformAirControlForPathFollowing())
	{
		const FVector FallVelocity = MoveVelocity.GetClampedToMaxSize(GetMaxSpeed());
		PerformAirControlForPathFollowing(FallVelocity, GetGravitySpaceZ(FallVelocity));
		return;
	}

	RequestedVelocity = MoveVelocity;
	bHasRequestedVelocity = true;
	bRequestedMoveWithMaxSpeed = bForceMaxSpeed;

	if (IsMovingOnGround())
	{
		RequestedVelocity = ProjectToGravityFloor(RequestedVelocity);
	}
}

bool UCharacterMovementComponent::ShouldPerformAirControlForPathFollowing() const
{
	return IsFalling();
}

void UCharacterMovementComponent::RequestPathMove(const FVector& MoveInput)
{
	FVector AdjustedMoveInput(MoveInput);

	// preserve magnitude when moving on ground/falling and requested input has Z component
	// see ConstrainInputAcceleration for details
	if (MoveInput.Z != 0.f && (IsMovingOnGround() || IsFalling()))
	{
		const float Mag = MoveInput.Size();
		AdjustedMoveInput = MoveInput.GetSafeNormal2D() * Mag;
	}
	
	Super::RequestPathMove(AdjustedMoveInput);
}

bool UCharacterMovementComponent::CanStartPathFollowing() const
{
	if (!HasValidData() || HasAnimRootMotion())
	{
		return false;
	}

	if (CharacterOwner)
	{
		if (CharacterOwner->GetRootComponent() && CharacterOwner->GetRootComponent()->IsSimulatingPhysics())
		{
			return false;
		}
	}

	return Super::CanStartPathFollowing();
}

bool UCharacterMovementComponent::CanStopPathFollowing() const
{
	return !IsFalling();
}

float UCharacterMovementComponent::GetPathFollowingBrakingDistance(float MaxSpeed) const
{
	if (NavMovementProperties.bUseFixedBrakingDistanceForPaths)
	{
		return NavMovementProperties.FixedPathBrakingDistance;
	}

	const float BrakingDeceleration = FMath::Abs(GetMaxBrakingDeceleration());

	// character won't be able to stop with negative or nearly zero deceleration, use MaxSpeed for path length calculations
	const float BrakingDistance = (BrakingDeceleration < UE_SMALL_NUMBER) ? MaxSpeed : (FMath::Square(MaxSpeed) / (2.f * BrakingDeceleration));
	return BrakingDistance;
}

void UCharacterMovementComponent::CalcAvoidanceVelocity(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_ObstacleAvoidance);

	UAvoidanceManager* AvoidanceManager = GetWorld()->GetAvoidanceManager();
	if (AvoidanceWeight >= 1.0f || AvoidanceManager == NULL || GetCharacterOwner() == NULL)
	{
		return;
	}

	if (GetCharacterOwner()->GetLocalRole() != ROLE_Authority)
	{
		return;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const bool bShowDebug = AvoidanceManager->IsDebugEnabled(AvoidanceUID);
#endif

	//Adjust velocity only if we're in "Walking" mode. We should also check if we're dazed, being knocked around, maybe off-navmesh, etc.
	UCapsuleComponent *OurCapsule = GetCharacterOwner()->GetCapsuleComponent();
	if (!Velocity.IsZero() && IsMovingOnGround() && OurCapsule)
	{
		//See if we're doing a locked avoidance move already, and if so, skip the testing and just do the move.
		if (AvoidanceLockTimer > 0.0f)
		{
			Velocity = AvoidanceLockVelocity;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bShowDebug)
			{
				DrawDebugLine(GetWorld(), GetActorFeetLocation(), GetActorFeetLocation() + Velocity, FColor::Blue, false, 0.5f, SDPG_MAX);
			}
#endif
		}
		else
		{
			FVector NewVelocity = AvoidanceManager->GetAvoidanceVelocityForComponent(this);
			if (bUseRVOPostProcess)
			{
				PostProcessAvoidanceVelocity(NewVelocity);
			}

			if (!NewVelocity.Equals(Velocity))		//Really want to branch hint that this will probably not pass
			{
				//Had to divert course, lock this avoidance move in for a short time. This will make us a VO, so unlocked others will know to avoid us.
				Velocity = NewVelocity;
				SetAvoidanceVelocityLock(AvoidanceManager, AvoidanceManager->LockTimeAfterAvoid);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bShowDebug)
				{
					DrawDebugLine(GetWorld(), GetActorFeetLocation(), GetActorFeetLocation() + Velocity, FColor::Red, false, 0.05f, SDPG_MAX, 10.0f);
				}
#endif
			}
			else
			{
				//Although we didn't divert course, our velocity for this frame is decided. We will not reciprocate anything further, so treat as a VO for the remainder of this frame.
				SetAvoidanceVelocityLock(AvoidanceManager, AvoidanceManager->LockTimeAfterClean);	//10 ms of lock time should be adequate.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bShowDebug)
				{
					//DrawDebugLine(GetWorld(), GetActorLocation(), GetActorLocation() + Velocity, FColor::Green, false, 0.05f, SDPG_MAX, 10.0f);
				}
#endif
			}
		}
		//RickH - We might do better to do this later in our update
		AvoidanceManager->UpdateRVO(this);

		bWasAvoidanceUpdated = true;
	}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	else if (bShowDebug)
	{
		DrawDebugLine(GetWorld(), GetActorFeetLocation(), GetActorFeetLocation() + Velocity, FColor::Yellow, false, 0.05f, SDPG_MAX);
	}

	if (bShowDebug)
	{
		DrawDebugLine(GetWorld(), GetActorFeetLocation(), GetActorFeetLocation() + 500.f * -GetGravityDirection(), (AvoidanceLockTimer > 0.01f) ? FColor::Red : FColor::Blue, false, 0.05f, SDPG_MAX, 5.0f);
	}
#endif
}

void UCharacterMovementComponent::PostProcessAvoidanceVelocity(FVector& NewVelocity)
{
	// empty in base class
}

void UCharacterMovementComponent::UpdateDefaultAvoidance()
{
	if (!bUseRVOAvoidance)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_AI_ObstacleAvoidance);

	UAvoidanceManager* AvoidanceManager = GetWorld()->GetAvoidanceManager();
	if (AvoidanceManager && !bWasAvoidanceUpdated && GetCharacterOwner()->GetCapsuleComponent())
	{
		AvoidanceManager->UpdateRVO(this);

		//Consider this a clean move because we didn't even try to avoid.
		SetAvoidanceVelocityLock(AvoidanceManager, AvoidanceManager->LockTimeAfterClean);
	}

	bWasAvoidanceUpdated = false;		//Reset for next frame
}

void UCharacterMovementComponent::SetRVOAvoidanceUID(int32 UID)
{
	AvoidanceUID = UID;
}

int32 UCharacterMovementComponent::GetRVOAvoidanceUID()
{
	return AvoidanceUID;
}

void UCharacterMovementComponent::SetRVOAvoidanceWeight(float Weight)
{
	AvoidanceWeight = Weight;
}

float UCharacterMovementComponent::GetRVOAvoidanceWeight()
{
	return AvoidanceWeight;
}

FVector UCharacterMovementComponent::GetRVOAvoidanceOrigin()
{
	return GetActorFeetLocation();
}

float UCharacterMovementComponent::GetRVOAvoidanceRadius()
{
	UCapsuleComponent* CapsuleComp = GetCharacterOwner()->GetCapsuleComponent();
	return CapsuleComp ? CapsuleComp->GetScaledCapsuleRadius() : 0.0f;
}

float UCharacterMovementComponent::GetRVOAvoidanceConsiderationRadius()
{
	return AvoidanceConsiderationRadius;
}

float UCharacterMovementComponent::GetRVOAvoidanceHeight()
{
	UCapsuleComponent* CapsuleComp = GetCharacterOwner()->GetCapsuleComponent();
	return CapsuleComp ? CapsuleComp->GetScaledCapsuleHalfHeight() : 0.0f;
}

FVector UCharacterMovementComponent::GetVelocityForRVOConsideration()
{
	return Velocity;
}

void UCharacterMovementComponent::SetAvoidanceGroupMask(int32 GroupFlags)
{
	AvoidanceGroup.SetFlagsDirectly(GroupFlags);
}

int32 UCharacterMovementComponent::GetAvoidanceGroupMask()
{
	return AvoidanceGroup.Packed;
}

void UCharacterMovementComponent::SetGroupsToAvoidMask(int32 GroupFlags)
{
	GroupsToAvoid.SetFlagsDirectly(GroupFlags);
}

int32 UCharacterMovementComponent::GetGroupsToAvoidMask()
{
	return GroupsToAvoid.Packed;
}

void UCharacterMovementComponent::SetGroupsToIgnoreMask(int32 GroupFlags)
{
	GroupsToIgnore.SetFlagsDirectly(GroupFlags);
}

int32 UCharacterMovementComponent::GetGroupsToIgnoreMask()
{
	return GroupsToIgnore.Packed;
}

void UCharacterMovementComponent::SetAvoidanceVelocityLock(class UAvoidanceManager* Avoidance, float Duration)
{
	Avoidance->OverrideToMaxWeight(AvoidanceUID, Duration);
	AvoidanceLockVelocity = Velocity;
	AvoidanceLockTimer = Duration;
}

void UCharacterMovementComponent::NotifyBumpedPawn(APawn* BumpedPawn)
{
	Super::NotifyBumpedPawn(BumpedPawn);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UAvoidanceManager* Avoidance = GetWorld()->GetAvoidanceManager();
	const bool bShowDebug = Avoidance && Avoidance->IsDebugEnabled(AvoidanceUID);
	if (bShowDebug)
	{
		DrawDebugLine(GetWorld(), GetActorFeetLocation(), GetActorFeetLocation() + 500.f * -GetGravityDirection(), (AvoidanceLockTimer > 0) ? FColor(255,64,64) : FColor(64,64,255), false, 2.0f, SDPG_MAX, 20.0f);
	}
#endif

	// Unlock avoidance move. This mostly happens when two pawns who are locked into avoidance moves collide with each other.
	AvoidanceLockTimer = 0.0f;
}

float UCharacterMovementComponent::GetMaxJumpHeight() const
{
	const float Gravity = GetGravityZ();
	if (FMath::Abs(Gravity) > UE_KINDA_SMALL_NUMBER)
	{
		return FMath::Square(JumpZVelocity) / (-2.f * Gravity);
	}
	else
	{
		return 0.f;
	}
}

float UCharacterMovementComponent::GetMaxJumpHeightWithJumpTime() const
{
	const float MaxJumpHeight = GetMaxJumpHeight();

	if (CharacterOwner)
	{
		// When bApplyGravityWhileJumping is true, the actual max height will be lower than this.
		// However, it will also be dependent on framerate (and substep iterations) so just return this
		// to avoid expensive calculations.

		// This can be imagined as the character being displaced to some height, then jumping from that height.
		return (CharacterOwner->JumpMaxHoldTime * JumpZVelocity) + MaxJumpHeight;
	}

	return MaxJumpHeight;
}

float UCharacterMovementComponent::GetMaxAcceleration() const
{
	return MaxAcceleration;
}

float UCharacterMovementComponent::GetMaxBrakingDeceleration() const
{
	switch (MovementMode)
	{
		case MOVE_Walking:
		case MOVE_NavWalking:
			return BrakingDecelerationWalking;
		case MOVE_Falling:
			return BrakingDecelerationFalling;
		case MOVE_Swimming:
			return BrakingDecelerationSwimming;
		case MOVE_Flying:
			return BrakingDecelerationFlying;
		case MOVE_Custom:
			return 0.f;
		case MOVE_None:
		default:
			return 0.f;
	}
}

FVector UCharacterMovementComponent::GetCurrentAcceleration() const
{
	return Acceleration;
}

void UCharacterMovementComponent::ApplyVelocityBraking(float DeltaTime, float Friction, float BrakingDeceleration)
{
	if (Velocity.IsZero() || !HasValidData() || HasAnimRootMotion() || DeltaTime < MIN_TICK_TIME)
	{
		return;
	}

	const float FrictionFactor = FMath::Max(0.f, BrakingFrictionFactor);
	Friction = FMath::Max(0.f, Friction * FrictionFactor);
	BrakingDeceleration = FMath::Max(0.f, BrakingDeceleration);
	const bool bZeroFriction = (Friction == 0.f);
	const bool bZeroBraking = (BrakingDeceleration == 0.f);

	if (bZeroFriction && bZeroBraking)
	{
		return;
	}

	const FVector OldVel = Velocity;

	// subdivide braking to get reasonably consistent results at lower frame rates
	// (important for packet loss situations w/ networking)
	float RemainingTime = DeltaTime;
	const float MaxTimeStep = FMath::Clamp(BrakingSubStepTime, 1.0f / 75.0f, 1.0f / 20.0f);

	// Decelerate to brake to a stop
	const FVector RevAccel = (bZeroBraking ? FVector::ZeroVector : (-BrakingDeceleration * Velocity.GetSafeNormal()));
	while( RemainingTime >= MIN_TICK_TIME )
	{
		// Zero friction uses constant deceleration, so no need for iteration.
		const float dt = ((RemainingTime > MaxTimeStep && !bZeroFriction) ? FMath::Min(MaxTimeStep, RemainingTime * 0.5f) : RemainingTime);
		RemainingTime -= dt;

		// apply friction and braking
		Velocity = Velocity + ((-Friction) * Velocity + RevAccel) * dt ; 
		
		// Don't reverse direction
		if ((Velocity | OldVel) <= 0.f)
		{
			Velocity = FVector::ZeroVector;
			return;
		}
	}

	// Clamp to zero if nearly zero, or if below min threshold and braking.
	const float VSizeSq = Velocity.SizeSquared();
	if (VSizeSq <= UE_KINDA_SMALL_NUMBER || (!bZeroBraking && VSizeSq <= FMath::Square(BRAKE_TO_STOP_VELOCITY)))
	{
		Velocity = FVector::ZeroVector;
	}
}

void UCharacterMovementComponent::PhysFlying(float deltaTime, int32 Iterations)
{
	if (deltaTime < MIN_TICK_TIME)
	{
		return;
	}

	RestorePreAdditiveRootMotionVelocity();

	if( !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() )
	{
		if( bCheatFlying && Acceleration.IsZero() )
		{
			Velocity = FVector::ZeroVector;
		}
		const float Friction = 0.5f * GetPhysicsVolume()->FluidFriction;
		CalcVelocity(deltaTime, Friction, true, GetMaxBrakingDeceleration());
	}

	ApplyRootMotionToVelocity(deltaTime);

	Iterations++;
	bJustTeleported = false;

	FVector OldLocation = UpdatedComponent->GetComponentLocation();
	const FVector Adjusted = Velocity * deltaTime;
	FHitResult Hit(1.f);
	SafeMoveUpdatedComponent(Adjusted, UpdatedComponent->GetComponentQuat(), true, Hit);

	if (Hit.Time < 1.f)
	{
		const FVector VelDir = Velocity.GetSafeNormal();
		const float UpDown = VelDir | GetGravityDirection();

		bool bSteppedUp = false;
		if ((FMath::Abs(GetGravitySpaceZ(Hit.ImpactNormal)) < 0.2f) && (UpDown < 0.5f) && (UpDown > -0.2f) && CanStepUp(Hit))
		{
			const FVector::FReal StepZ = GetGravitySpaceZ(UpdatedComponent->GetComponentLocation());
			bSteppedUp = StepUp(GetGravityDirection(), Adjusted * (1.f - Hit.Time), Hit);
			if (bSteppedUp)
			{
				const FVector::FReal LocationZ = GetGravitySpaceZ(UpdatedComponent->GetComponentLocation()) + (GetGravitySpaceZ(OldLocation) - StepZ);
				SetGravitySpaceZ(OldLocation, LocationZ);
			}
		}

		if (!bSteppedUp)
		{
			//adjust and try again
			HandleImpact(Hit, deltaTime, Adjusted);
			SlideAlongSurface(Adjusted, (1.f - Hit.Time), Hit.Normal, Hit, true);
		}
	}

	if( !bJustTeleported && !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() )
	{
		Velocity = (UpdatedComponent->GetComponentLocation() - OldLocation) / deltaTime;
	}
}

void UCharacterMovementComponent::RestorePreAdditiveRootMotionVelocity()
{
	// Restore last frame's pre-additive Velocity if we had additive applied 
	// so that we're not adding more additive velocity than intended
	if( CurrentRootMotion.bIsAdditiveVelocityApplied )
	{
#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
		{
			FString AdjustedDebugString = FString::Printf(TEXT("RestorePreAdditiveRootMotionVelocity Velocity(%s) LastPreAdditiveVelocity(%s)"), 
				*Velocity.ToCompactString(), *CurrentRootMotion.LastPreAdditiveVelocity.ToCompactString());
			RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
		}
#endif

		Velocity = CurrentRootMotion.LastPreAdditiveVelocity;
		CurrentRootMotion.bIsAdditiveVelocityApplied = false;
	}
}

void UCharacterMovementComponent::ApplyRootMotionToVelocity(float deltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementRootMotionSourceApply);

	// Animation root motion is distinct from root motion sources right now and takes precedence
	if( HasAnimRootMotion() && deltaTime > 0.f )
	{
		Velocity = ConstrainAnimRootMotionVelocity(AnimRootMotionVelocity, Velocity);
		if (IsFalling())
		{
			Velocity += ProjectToGravityFloor(DecayingFormerBaseVelocity);
		}
		return;
	}

	const FVector OldVelocity = Velocity;

	bool bAppliedRootMotion = false;

	// Apply override velocity
	if( CurrentRootMotion.HasOverrideVelocity() )
	{
		CurrentRootMotion.AccumulateOverrideRootMotionVelocity(deltaTime, *CharacterOwner, *this, Velocity);
		if (IsFalling())
		{
			Velocity += CurrentRootMotion.HasOverrideVelocityWithIgnoreZAccumulate() ? ProjectToGravityFloor(DecayingFormerBaseVelocity) : DecayingFormerBaseVelocity;
		}
		bAppliedRootMotion = true;

#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
		{
			FString AdjustedDebugString = FString::Printf(TEXT("ApplyRootMotionToVelocity HasOverrideVelocity Velocity(%s)"),
				*Velocity.ToCompactString());
			RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
		}
#endif
	}

	// Next apply additive root motion
	if( CurrentRootMotion.HasAdditiveVelocity() )
	{
		CurrentRootMotion.LastPreAdditiveVelocity = Velocity; // Save off pre-additive Velocity for restoration next tick
		CurrentRootMotion.AccumulateAdditiveRootMotionVelocity(deltaTime, *CharacterOwner, *this, Velocity);
		CurrentRootMotion.bIsAdditiveVelocityApplied = true; // Remember that we have it applied
		bAppliedRootMotion = true;

#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
		{
			FString AdjustedDebugString = FString::Printf(TEXT("ApplyRootMotionToVelocity HasAdditiveVelocity Velocity(%s) LastPreAdditiveVelocity(%s)"),
				*Velocity.ToCompactString(), *CurrentRootMotion.LastPreAdditiveVelocity.ToCompactString());
			RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
		}
#endif
	}

	// Switch to Falling if we have vertical velocity from root motion so we can lift off the ground
	const FVector AppliedVelocityDelta = Velocity - OldVelocity;
	if( bAppliedRootMotion && GetGravitySpaceZ(AppliedVelocityDelta) != 0.f && IsMovingOnGround() )
	{
		float LiftoffBound;
		if( CurrentRootMotion.LastAccumulatedSettings.HasFlag(ERootMotionSourceSettingsFlags::UseSensitiveLiftoffCheck) )
		{
			// Sensitive bounds - "any positive force"
			LiftoffBound = UE_SMALL_NUMBER;
		}
		else
		{
			// Default bounds - the amount of force gravity is applying this tick
			LiftoffBound = FMath::Max(-GetGravityZ() * deltaTime, UE_SMALL_NUMBER);
		}

		if(GetGravitySpaceZ(AppliedVelocityDelta) > LiftoffBound )
		{
			SetMovementMode(MOVE_Falling);
		}
	}
}

void UCharacterMovementComponent::DecayFormerBaseVelocity(float deltaTime)
{
	if (!CharacterMovementCVars::bAddFormerBaseVelocityToRootMotionOverrideWhenFalling || FormerBaseVelocityDecayHalfLife == 0.f)
	{
		DecayingFormerBaseVelocity = FVector::ZeroVector;
	}
	else if (FormerBaseVelocityDecayHalfLife > 0.f)
	{
		DecayingFormerBaseVelocity *= FMath::Exp2(-deltaTime * 1.f / FormerBaseVelocityDecayHalfLife);
	}
}

void UCharacterMovementComponent::HandleSwimmingWallHit(const FHitResult& Hit, float DeltaTime)
{
}

void UCharacterMovementComponent::PhysSwimming(float deltaTime, int32 Iterations)
{
	if (deltaTime < MIN_TICK_TIME)
	{
		return;
	}

	RestorePreAdditiveRootMotionVelocity();

	float NetFluidFriction  = 0.f;
	float Depth = ImmersionDepth();
	float NetBuoyancy = Buoyancy * Depth;
	float OriginalAccelZ = GetGravitySpaceZ(Acceleration);
	bool bLimitedUpAccel = false;

	if (!HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() && (GetGravitySpaceZ(Velocity) > 0.33f * MaxSwimSpeed) && (NetBuoyancy != 0.f))
	{
		//damp positive Z out of water
		SetGravitySpaceZ(Velocity, FMath::Max<FVector::FReal>(0.33f * MaxSwimSpeed, GetGravitySpaceZ(Velocity) * Depth*Depth));
	}
	else if (Depth < 0.65f)
	{
		bLimitedUpAccel = (OriginalAccelZ > 0.f);
		SetGravitySpaceZ(Acceleration, FMath::Min<FVector::FReal>(0.1f, OriginalAccelZ));
	}

	Iterations++;
	FVector OldLocation = UpdatedComponent->GetComponentLocation();
	bJustTeleported = false;
	if( !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() )
	{
		const float Friction = 0.5f * GetPhysicsVolume()->FluidFriction * Depth;
		CalcVelocity(deltaTime, Friction, true, GetMaxBrakingDeceleration());
		Velocity += (GetGravityZ() * deltaTime * (1.f - NetBuoyancy)) * -GetGravityDirection();
	}

	ApplyRootMotionToVelocity(deltaTime);

	FVector Adjusted = Velocity * deltaTime;
	FHitResult Hit(1.f);
	const float remainingTime = deltaTime * Swim(Adjusted, Hit);

	//may have left water - if so, script might have set new physics mode
	if ( !IsSwimming() )
	{
		StartNewPhysics(remainingTime, Iterations);
		return;
	}

	if ( Hit.Time < 1.f && CharacterOwner)
	{
		HandleSwimmingWallHit(Hit, deltaTime);
		if (bLimitedUpAccel && (GetGravitySpaceZ(Velocity) >= 0.f))
		{
			// allow upward velocity at surface if against obstacle
			Velocity += OriginalAccelZ * deltaTime * -GetGravityDirection();
			Adjusted = Velocity * (1.f - Hit.Time)*deltaTime;
			Swim(Adjusted, Hit);
			if (!IsSwimming())
			{
				StartNewPhysics(remainingTime, Iterations);
				return;
			}
		}

		const FVector VelDir = Velocity.GetSafeNormal();
		const float UpDown = VelDir | GetGravityDirection();

		bool bSteppedUp = false;
		if( (FMath::Abs(GetGravitySpaceZ(Hit.ImpactNormal)) < 0.2f) && (UpDown < 0.5f) && (UpDown > -0.2f) && CanStepUp(Hit))
		{
			const float StepZ = GetGravitySpaceZ(UpdatedComponent->GetComponentLocation());
			const FVector RealVelocity = Velocity;
			SetGravitySpaceZ(Velocity, 1.f);	// HACK: since will be moving up, in case pawn leaves the water
			bSteppedUp = StepUp(GetGravityDirection(), Adjusted * (1.f - Hit.Time), Hit);
			if (bSteppedUp)
			{
				//may have left water - if so, script might have set new physics mode
				if (!IsSwimming())
				{
					StartNewPhysics(remainingTime, Iterations);
					return;
				}
				SetGravitySpaceZ(OldLocation, GetGravitySpaceZ(UpdatedComponent->GetComponentLocation()) + (GetGravitySpaceZ(OldLocation) - StepZ));
			}
			Velocity = RealVelocity;
		}

		if (!bSteppedUp)
		{
			//adjust and try again
			HandleImpact(Hit, deltaTime, Adjusted);
			SlideAlongSurface(Adjusted, (1.f - Hit.Time), Hit.Normal, Hit, true);
		}
	}

	if( !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() && !bJustTeleported && ((deltaTime - remainingTime) > UE_KINDA_SMALL_NUMBER) && CharacterOwner )
	{
		const bool bWaterJump = !GetPhysicsVolume()->bWaterVolume;
		const FVector::FReal VelZ = GetGravitySpaceZ(Velocity);
		Velocity = (UpdatedComponent->GetComponentLocation() - OldLocation) / (deltaTime - remainingTime);
		if (bWaterJump)
		{
			SetGravitySpaceZ(Velocity, VelZ);
		}
	}

	if ( !GetPhysicsVolume()->bWaterVolume && IsSwimming() )
	{
		SetMovementMode(MOVE_Falling); //in case script didn't change it (w/ zone change)
	}

	//may have left water - if so, script might have set new physics mode
	if ( !IsSwimming() )
	{
		StartNewPhysics(remainingTime, Iterations);
	}
}


void UCharacterMovementComponent::StartSwimming(FVector OldLocation, FVector OldVelocity, float timeTick, float remainingTime, int32 Iterations)
{
	if (remainingTime < MIN_TICK_TIME || timeTick < MIN_TICK_TIME)
	{
		return;
	}

	if( !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() && !bJustTeleported )
	{
		Velocity = (UpdatedComponent->GetComponentLocation() - OldLocation)/timeTick; //actual average velocity
		Velocity = 2.f*Velocity - OldVelocity; //end velocity has 2* accel of avg
		Velocity = Velocity.GetClampedToMaxSize(GetPhysicsVolume()->TerminalVelocity);
	}
	const FVector End = FindWaterLine(UpdatedComponent->GetComponentLocation(), OldLocation);
	float waterTime = 0.f;
	if (End != UpdatedComponent->GetComponentLocation())
	{	
		const float ActualDist = (UpdatedComponent->GetComponentLocation() - OldLocation).Size();
		if (ActualDist > UE_KINDA_SMALL_NUMBER)
		{
			waterTime = timeTick * (End - UpdatedComponent->GetComponentLocation()).Size() / ActualDist;
			remainingTime += waterTime;
		}
		MoveUpdatedComponent(End - UpdatedComponent->GetComponentLocation(), UpdatedComponent->GetComponentQuat(), true);
	}
	const FVector::FReal GravityRelativeVelocityZ = GetGravitySpaceZ(Velocity);
	if ( !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() && (GravityRelativeVelocityZ > 2.f*CharacterMovementConstants::SWIMBOBSPEED) && (GravityRelativeVelocityZ < 0.f)) //allow for falling out of water
	{
		SetGravitySpaceZ(Velocity, CharacterMovementConstants::SWIMBOBSPEED - ProjectToGravityFloor(Velocity).Size() * 0.7f); //smooth bobbing
	}
	if ( (remainingTime >= MIN_TICK_TIME) && (Iterations < MaxSimulationIterations) )
	{
		PhysSwimming(remainingTime, Iterations);
	}
}

float UCharacterMovementComponent::Swim(FVector Delta, FHitResult& Hit)
{
	FVector Start = UpdatedComponent->GetComponentLocation();
	float airTime = 0.f;
	SafeMoveUpdatedComponent(Delta, UpdatedComponent->GetComponentQuat(), true, Hit);

	if ( !GetPhysicsVolume()->bWaterVolume ) //then left water
	{
		const FVector End = FindWaterLine(Start,UpdatedComponent->GetComponentLocation());
		const float DesiredDist = Delta.Size();
		if (End != UpdatedComponent->GetComponentLocation() && DesiredDist > UE_KINDA_SMALL_NUMBER)
		{
			airTime = (End - UpdatedComponent->GetComponentLocation()).Size() / DesiredDist;
			if ( ((UpdatedComponent->GetComponentLocation() - Start) | (End - UpdatedComponent->GetComponentLocation())) > 0.f )
			{
				airTime = 0.f;
			}
			SafeMoveUpdatedComponent(End - UpdatedComponent->GetComponentLocation(), UpdatedComponent->GetComponentQuat(), true, Hit);
		}
	}
	return airTime;
}

FVector UCharacterMovementComponent::FindWaterLine(FVector InWater, FVector OutofWater)
{
	FVector Result = OutofWater;

	if (!IsValid(CharacterOwner))
	{
		return Result;
	}

	TArray<FHitResult> Hits;
	GetWorld()->LineTraceMultiByChannel(Hits, OutofWater, InWater, UpdatedComponent->GetCollisionObjectType(), FCollisionQueryParams(SCENE_QUERY_STAT(FindWaterLine), true, CharacterOwner));

	for (const FHitResult& Check : Hits)
	{
		const UPrimitiveComponent* HitComponent = Check.Component.Get();
		if (!CharacterOwner->IsOwnedBy(Check.HitObjectHandle.FetchActor()) && HitComponent && !HitComponent->IsWorldGeometry())
		{
			APhysicsVolume *W = Check.HitObjectHandle.FetchActor<APhysicsVolume>();
			if ( W && W->bWaterVolume )
			{
				FVector Dir = (InWater - OutofWater).GetSafeNormal();
				Result = Check.Location;
				if ( W == GetPhysicsVolume() )
					Result += 0.1f * Dir;
				else
					Result -= 0.1f * Dir;
				break;
			}
		}
	}

	return Result;
}

void UCharacterMovementComponent::NotifyJumpApex() 
{
	if( CharacterOwner )
	{
		CharacterOwner->NotifyJumpApex();
	}
}


FVector UCharacterMovementComponent::GetFallingLateralAcceleration(float DeltaTime)
{
	// No acceleration in Z
	FVector FallAcceleration = ProjectToGravityFloor(Acceleration);

	// bound acceleration, falling object has minimal ability to impact acceleration
	if (!HasAnimRootMotion() && FallAcceleration.SizeSquared() > 0.f)
	{
		FallAcceleration = GetAirControl(DeltaTime, AirControl, FallAcceleration);
		FallAcceleration = FallAcceleration.GetClampedToMaxSize(GetMaxAcceleration());
	}

	return FallAcceleration;
}


bool UCharacterMovementComponent::ShouldLimitAirControl(float DeltaTime, const FVector& FallAcceleration) const
{
	return (ProjectToGravityFloor(FallAcceleration).SizeSquared() > 0.f);
}

FVector UCharacterMovementComponent::GetAirControl(float DeltaTime, float TickAirControl, const FVector& FallAcceleration)
{
	// Boost
	if (TickAirControl != 0.f)
	{
		TickAirControl = BoostAirControl(DeltaTime, TickAirControl, FallAcceleration);
	}

	return TickAirControl * FallAcceleration;
}

void UCharacterMovementComponent::SetGravityDirection(const FVector& InNewGravityDir)
{
	FVector NewGravityDir = InNewGravityDir.GetSafeNormal();
	if (ensure(!NewGravityDir.IsNearlyZero()))
	{
		if (!GravityDirection.Equals(NewGravityDir))
		{
			UE_LOGF(LogCharacterMovement, Verbose, "SetGravityDirection: From(%ls) To(%ls)", *GravityDirection.ToCompactString(), *NewGravityDir.ToCompactString());
			GravityDirection = NewGravityDir;
			WorldToGravityTransform = FQuat::FindBetweenNormals(FVector::UpVector, -NewGravityDir);
			GravityToWorldTransform = WorldToGravityTransform.Inverse();
			bHasCustomGravity = !GravityDirection.Equals(DefaultGravityDirection);
		}
	}
}

float UCharacterMovementComponent::BoostAirControl(float DeltaTime, float TickAirControl, const FVector& FallAcceleration)
{
	// Allow a burst of initial acceleration
	if (AirControlBoostMultiplier > 0.f && ProjectToGravityFloor(Velocity).SizeSquared() < FMath::Square(AirControlBoostVelocityThreshold))
	{
		TickAirControl = FMath::Min(1.f, AirControlBoostMultiplier * TickAirControl);
	}

	return TickAirControl;
}


void UCharacterMovementComponent::PhysFalling(float deltaTime, int32 Iterations)
{
	SCOPE_CYCLE_COUNTER(STAT_CharPhysFalling);

	if (deltaTime < MIN_TICK_TIME)
	{
		return;
	}

	const FVector FallAcceleration = ProjectToGravityFloor(GetFallingLateralAcceleration(deltaTime));
	const bool bHasLimitedAirControl = ShouldLimitAirControl(deltaTime, FallAcceleration);

	float remainingTime = deltaTime;
	while( (remainingTime >= MIN_TICK_TIME) && (Iterations < MaxSimulationIterations) )
	{
		Iterations++;
		float timeTick = GetSimulationTimeStep(remainingTime, Iterations);
		remainingTime -= timeTick;
		
		const FVector OldLocation = UpdatedComponent->GetComponentLocation();
		const FQuat PawnRotation = UpdatedComponent->GetComponentQuat();
		bJustTeleported = false;

		const FVector OldVelocityWithRootMotion = Velocity;

		RestorePreAdditiveRootMotionVelocity();

		const FVector OldVelocity = Velocity;

		// Apply input
		const float MaxDecel = GetMaxBrakingDeceleration();
		if (!HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity())
		{
			// Compute Velocity
			{
				// Acceleration = FallAcceleration for CalcVelocity(), but we restore it after using it.
				TGuardValue<FVector> RestoreAcceleration(Acceleration, FallAcceleration);
				if (HasCustomGravity())
				{
					Velocity = ProjectToGravityFloor(Velocity);
					const FVector GravityRelativeOffset = OldVelocity - Velocity;
					CalcVelocity(timeTick, FallingLateralFriction, false, MaxDecel);
					Velocity += GravityRelativeOffset;
				}
				else
				{
					Velocity.Z = 0.f;
					CalcVelocity(timeTick, FallingLateralFriction, false, MaxDecel);
					Velocity.Z = OldVelocity.Z;
				}
			}
		}

		// Compute current gravity
		const FVector Gravity = -GetGravityDirection() * GetGravityZ();
		float GravityTime = timeTick;

		// If jump is providing force, gravity may be affected.
		bool bEndingJumpForce = false;
		if (CharacterOwner->JumpForceTimeRemaining > 0.0f)
		{
			// Consume some of the force time. Only the remaining time (if any) is affected by gravity when bApplyGravityWhileJumping=false.
			const float JumpForceTime = FMath::Min(CharacterOwner->JumpForceTimeRemaining, timeTick);
			GravityTime = bApplyGravityWhileJumping ? timeTick : FMath::Max(0.0f, timeTick - JumpForceTime);
			
			// Update Character state
			CharacterOwner->JumpForceTimeRemaining -= JumpForceTime;
			if (CharacterOwner->JumpForceTimeRemaining <= 0.0f)
			{
				CharacterOwner->ResetJumpState();
				bEndingJumpForce = true;
			}
		}

		// Apply gravity
		Velocity = NewFallVelocity(Velocity, Gravity, GravityTime);

		//UE_LOGF(LogCharacterMovement, Log, "dt=(%.6f) OldLocation=(%ls) OldVelocity=(%ls) OldVelocityWithRootMotion=(%ls) NewVelocity=(%ls)", timeTick, *(UpdatedComponent->GetComponentLocation()).ToString(), *OldVelocity.ToString(), *OldVelocityWithRootMotion.ToString(), *Velocity.ToString());
		ApplyRootMotionToVelocity(timeTick);
		DecayFormerBaseVelocity(timeTick);

		// See if we need to sub-step to exactly reach the apex. This is important for avoiding "cutting off the top" of the trajectory as framerate varies.
		const FVector::FReal GravityRelativeOldVelocityWithRootMotionZ = GetGravitySpaceZ(OldVelocityWithRootMotion);
		if (CharacterMovementCVars::ForceJumpPeakSubstep && GravityRelativeOldVelocityWithRootMotionZ > 0.f && GetGravitySpaceZ(Velocity) <= 0.f && NumJumpApexAttempts < MaxJumpApexAttemptsPerSimulation)
		{
			const FVector DerivedAccel = (Velocity - OldVelocityWithRootMotion) / timeTick;
			const FVector::FReal GravityRelativeDerivedAccelZ = GetGravitySpaceZ(DerivedAccel);
			if (!FMath::IsNearlyZero(GravityRelativeDerivedAccelZ))
			{
				const float TimeToApex = -GravityRelativeOldVelocityWithRootMotionZ / GravityRelativeDerivedAccelZ;
				
				// The time-to-apex calculation should be precise, and we want to avoid adding a substep when we are basically already at the apex from the previous iteration's work.
				const float ApexTimeMinimum = 0.0001f;
				if (TimeToApex >= ApexTimeMinimum && TimeToApex < timeTick)
				{
					const FVector ApexVelocity = OldVelocityWithRootMotion + (DerivedAccel * TimeToApex);
					if (HasCustomGravity())
					{
						Velocity = ProjectToGravityFloor(ApexVelocity); // Should be nearly zero anyway, but this makes apex notifications consistent.
					}
					else
					{
						Velocity = ApexVelocity;
						Velocity.Z = 0.f; // Should be nearly zero anyway, but this makes apex notifications consistent.
					}
					
					// We only want to move the amount of time it takes to reach the apex, and refund the unused time for next iteration.
					const float TimeToRefund = (timeTick - TimeToApex);

					remainingTime += TimeToRefund;
					timeTick = TimeToApex;
					Iterations--;
					NumJumpApexAttempts++;

					// Refund time to any active Root Motion Sources as well
					for (TSharedPtr<FRootMotionSource> RootMotionSource : CurrentRootMotion.RootMotionSources)
					{
						const float RewoundRMSTime = FMath::Max(0.0f, RootMotionSource->GetTime() - TimeToRefund);
						RootMotionSource->SetTime(RewoundRMSTime);
					}
				}
			}
		}

		if (bNotifyApex && (GetGravitySpaceZ(Velocity) < 0.f))
		{
			// Just passed jump apex since now going down
			bNotifyApex = false;
			NotifyJumpApex();
		}

		// Compute change in position (using midpoint integration method).
		FVector Adjusted = 0.5f * (OldVelocityWithRootMotion + Velocity) * timeTick;
		
		// Special handling if ending the jump force where we didn't apply gravity during the jump.
		if (bEndingJumpForce && !bApplyGravityWhileJumping)
		{
			// We had a portion of the time at constant speed then a portion with acceleration due to gravity.
			// Account for that here with a more correct change in position.
			const float NonGravityTime = FMath::Max(0.f, timeTick - GravityTime);
			Adjusted = (OldVelocityWithRootMotion * NonGravityTime) + (0.5f*(OldVelocityWithRootMotion + Velocity) * GravityTime);
		}

		// Move
		FHitResult Hit(1.f);
		SafeMoveUpdatedComponent( Adjusted, PawnRotation, true, Hit);
		
		if (!HasValidData())
		{
			return;
		}
		
		float LastMoveTimeSlice = timeTick;
		float subTimeTickRemaining = timeTick * (1.f - Hit.Time);
		
		if ( IsSwimming() ) //just entered water
		{
			remainingTime += subTimeTickRemaining;
			StartSwimming(OldLocation, OldVelocity, timeTick, remainingTime, Iterations);
			return;
		}
		else if ( Hit.bBlockingHit )
		{
			if (IsValidLandingSpot(UpdatedComponent->GetComponentLocation(), Hit))
			{
				remainingTime += subTimeTickRemaining;
				ProcessLanded(Hit, remainingTime, Iterations);
				return;
			}
			else
			{
				// Compute impact deflection based on final velocity, not integration step.
				// This allows us to compute a new velocity from the deflected vector, and ensures the full gravity effect is included in the slide result.
				Adjusted = Velocity * timeTick;

				// See if we can convert a normally invalid landing spot (based on the hit result) to a usable one.
				if (!Hit.bStartPenetrating && ShouldCheckForValidLandingSpot(timeTick, Adjusted, Hit))
				{
					const FVector PawnLocation = UpdatedComponent->GetComponentLocation();
					FFindFloorResult FloorResult;
					FindFloor(PawnLocation, FloorResult, false);

					// Note that we only care about capsule sweep floor results, since the line trace may detect a lower walkable surface that our falling capsule wouldn't actually reach yet.
					if (!FloorResult.bLineTrace && FloorResult.IsWalkableFloor() && IsValidLandingSpot(PawnLocation, FloorResult.HitResult))
					{
						remainingTime += subTimeTickRemaining;
						ProcessLanded(FloorResult.HitResult, remainingTime, Iterations);
						return;
					}
				}

				HandleImpact(Hit, LastMoveTimeSlice, Adjusted);
				
				// If we've changed physics mode, abort.
				if (!HasValidData() || !IsFalling())
				{
					return;
				}

				// Limit air control based on what we hit.
				// We moved to the impact point using air control, but may want to deflect from there based on a limited air control acceleration.
				FVector VelocityNoAirControl = OldVelocity;
				FVector AirControlAccel = Acceleration;
				if (bHasLimitedAirControl)
				{
					// Compute VelocityNoAirControl
					{
						// Find velocity *without* acceleration.
						TGuardValue<FVector> RestoreAcceleration(Acceleration, FVector::ZeroVector);
						TGuardValue<FVector> RestoreVelocity(Velocity, OldVelocity);
						if (HasCustomGravity())
						{
							Velocity = ProjectToGravityFloor(Velocity);
							const FVector GravityRelativeOffset = OldVelocity - Velocity;
							CalcVelocity(timeTick, FallingLateralFriction, false, MaxDecel);
							VelocityNoAirControl = Velocity + GravityRelativeOffset;
						}
						else
						{
							Velocity.Z = 0.f;
							CalcVelocity(timeTick, FallingLateralFriction, false, MaxDecel);
							VelocityNoAirControl = FVector(Velocity.X, Velocity.Y, OldVelocity.Z);
						}
						
						VelocityNoAirControl = NewFallVelocity(VelocityNoAirControl, Gravity, GravityTime);
					}

					const bool bCheckLandingSpot = false; // we already checked above.
					AirControlAccel = (Velocity - VelocityNoAirControl) / timeTick;
					const FVector AirControlDeltaV = LimitAirControl(LastMoveTimeSlice, AirControlAccel, Hit, bCheckLandingSpot) * LastMoveTimeSlice;
					Adjusted = (VelocityNoAirControl + AirControlDeltaV) * LastMoveTimeSlice;
				}

				const FVector OldHitNormal = Hit.Normal;
				const FVector OldHitImpactNormal = Hit.ImpactNormal;				
				FVector Delta = ComputeSlideVector(Adjusted, 1.f - Hit.Time, OldHitNormal, Hit);

				// Compute velocity after deflection (only gravity component for RootMotion)
				FMovementBaseInterfaceData HitMovementBaseInterfaceData(MovementBaseUtility::GetMovementBaseDataFromHitResult(&Hit));
				
				if (CharacterMovementCVars::UseTargetVelocityOnImpact && !Velocity.IsNearlyZero() && MovementBaseUtility::IsSimulatedBase(&HitMovementBaseInterfaceData))
				{
					const FVector ContactVelocity = MovementBaseUtility::GetMovementBaseVelocity(&HitMovementBaseInterfaceData, NAME_None) + MovementBaseUtility::GetMovementBaseTangentialVelocity(&HitMovementBaseInterfaceData, NAME_None, Hit.ImpactPoint);
					const FVector NewVelocity = Velocity - Hit.ImpactNormal * FVector::DotProduct(Velocity - ContactVelocity, Hit.ImpactNormal);
					Velocity = HasAnimRootMotion() || CurrentRootMotion.HasOverrideVelocityWithIgnoreZAccumulate() ? ProjectToGravityFloor(Velocity) + GetGravitySpaceComponentZ(NewVelocity) : NewVelocity;
				}
				else if (subTimeTickRemaining > UE_KINDA_SMALL_NUMBER && !bJustTeleported)
				{
					const FVector NewVelocity = (Delta / subTimeTickRemaining);
					Velocity = HasAnimRootMotion() || CurrentRootMotion.HasOverrideVelocityWithIgnoreZAccumulate() ? ProjectToGravityFloor(Velocity) + GetGravitySpaceComponentZ(NewVelocity) : NewVelocity;
				}

				if (subTimeTickRemaining > UE_KINDA_SMALL_NUMBER && (Delta | Adjusted) > 0.f)
				{
					// Move in deflected direction.
					SafeMoveUpdatedComponent( Delta, PawnRotation, true, Hit);
					
					if (Hit.bBlockingHit)
					{
						// hit second wall
						LastMoveTimeSlice = subTimeTickRemaining;
						subTimeTickRemaining = subTimeTickRemaining * (1.f - Hit.Time);

						if (IsValidLandingSpot(UpdatedComponent->GetComponentLocation(), Hit))
						{
							remainingTime += subTimeTickRemaining;
							ProcessLanded(Hit, remainingTime, Iterations);
							return;
						}

						HandleImpact(Hit, LastMoveTimeSlice, Delta);

						// If we've changed physics mode, abort.
						if (!HasValidData() || !IsFalling())
						{
							return;
						}

						// Act as if there was no air control on the last move when computing new deflection.
						if (bHasLimitedAirControl && GetGravitySpaceZ(Hit.Normal) > CharacterMovementConstants::VERTICAL_SLOPE_NORMAL_Z)
						{
							const FVector LastMoveNoAirControl = VelocityNoAirControl * LastMoveTimeSlice;
							Delta = ComputeSlideVector(LastMoveNoAirControl, 1.f, OldHitNormal, Hit);
						}

						FVector PreTwoWallDelta = Delta;
						TwoWallAdjust(Delta, Hit, OldHitNormal);

						// Prevent extra upwards slope velocity that would occur when colliding inside a corner making an acute angle
						// For obtuse cases, HandleSlopeBoosting() is applied twice, but it stays within the first application's range, so this has no net effect.
						Delta = HandleSlopeBoosting(Delta, PreTwoWallDelta, 1.f - Hit.Time, Hit.Normal, Hit);

						// Limit air control, but allow a slide along the second wall.
						if (bHasLimitedAirControl)
						{
							const bool bCheckLandingSpot = false; // we already checked above.
							const FVector AirControlDeltaV = LimitAirControl(subTimeTickRemaining, AirControlAccel, Hit, bCheckLandingSpot) * subTimeTickRemaining;

							// Only allow if not back in to first wall
							if (FVector::DotProduct(AirControlDeltaV, OldHitNormal) > 0.f)
							{
								Delta += (AirControlDeltaV * subTimeTickRemaining);
							}
						}

						// Compute velocity after deflection (only gravity component for RootMotion)
						if (subTimeTickRemaining > UE_KINDA_SMALL_NUMBER && !bJustTeleported)
						{
							const FVector NewVelocity = (Delta / subTimeTickRemaining);
							Velocity = HasAnimRootMotion() || CurrentRootMotion.HasOverrideVelocityWithIgnoreZAccumulate() ? ProjectToGravityFloor(Velocity) + GetGravitySpaceComponentZ(NewVelocity) : NewVelocity;
						}

						// bDitch=true means that pawn is straddling two slopes, neither of which it can stand on
						bool bDitch = ( (GetGravitySpaceZ(OldHitImpactNormal) > 0.f) && (GetGravitySpaceZ(Hit.ImpactNormal) > 0.f) && (FMath::Abs(GetGravitySpaceZ(Delta)) <= UE_KINDA_SMALL_NUMBER) && ((Hit.ImpactNormal | OldHitImpactNormal) < 0.f) );
						SafeMoveUpdatedComponent( Delta, PawnRotation, true, Hit);
						if ( Hit.Time == 0.f )
						{
							// if we are stuck then try to side step
							FVector SideDelta = ProjectToGravityFloor(OldHitNormal + Hit.ImpactNormal).GetSafeNormal();
							if ( SideDelta.IsNearlyZero() )
							{
								if (HasCustomGravity())
								{
									const FVector GravityRelativeHitNormal = RotateWorldToGravity(OldHitNormal);
									SideDelta = RotateGravityToWorld(FVector(GravityRelativeHitNormal.Y, -GravityRelativeHitNormal.X, 0.f)).GetSafeNormal();
								}
								else
								{
									SideDelta = FVector(OldHitNormal.Y, -OldHitNormal.X, 0).GetSafeNormal();	
								}
							}
							SafeMoveUpdatedComponent( SideDelta, PawnRotation, true, Hit);
						}
							
						if ( bDitch || IsValidLandingSpot(UpdatedComponent->GetComponentLocation(), Hit) || Hit.Time == 0.f  )
						{
							remainingTime = 0.f;
							ProcessLanded(Hit, remainingTime, Iterations);
							return;
						}
						else if (GetPerchRadiusThreshold() > 0.f && Hit.Time == 1.f && GetGravitySpaceZ(OldHitImpactNormal) >= WalkableFloorZ)
						{
							// We might be in a virtual 'ditch' within our perch radius. This is rare.
							const FVector PawnLocation = UpdatedComponent->GetComponentLocation();
							const float ZMovedDist = FMath::Abs(GetGravitySpaceZ(PawnLocation - OldLocation));
							const float MovedDist2D = ProjectToGravityFloor(PawnLocation - OldLocation).Size();
							if (ZMovedDist <= 0.2f * timeTick && MovedDist2D <= 4.f * timeTick)
							{
								FVector GravityRelativeVelocity = RotateWorldToGravity(Velocity);
								GravityRelativeVelocity.X += 0.25f * GetMaxSpeed() * (RandomStream.FRand() - 0.5f);
								GravityRelativeVelocity.Y += 0.25f * GetMaxSpeed() * (RandomStream.FRand() - 0.5f);
								GravityRelativeVelocity.Z = FMath::Max<float>(JumpZVelocity * 0.25f, 1.f);
								Velocity = RotateGravityToWorld(GravityRelativeVelocity);
								Delta = Velocity * timeTick;
								SafeMoveUpdatedComponent(Delta, PawnRotation, true, Hit);
							}
						}
					}
				}
			}
		}

		const FVector GravityProjectedVelocity = ProjectToGravityFloor(Velocity);
		if (GravityProjectedVelocity.SizeSquared() <= UE_KINDA_SMALL_NUMBER * 10.f)
		{
			Velocity = GetGravitySpaceComponentZ(Velocity);
		}
	}
}

FVector UCharacterMovementComponent::LimitAirControl(float DeltaTime, const FVector& FallAcceleration, const FHitResult& HitResult, bool bCheckForValidLandingSpot)
{
	FVector Result(FallAcceleration);

	if (HitResult.IsValidBlockingHit() && GetGravitySpaceZ(HitResult.Normal) > CharacterMovementConstants::VERTICAL_SLOPE_NORMAL_Z)
	{
		if (!bCheckForValidLandingSpot || !IsValidLandingSpot(HitResult.Location, HitResult))
		{
			// If acceleration is into the wall, limit contribution.
			if (FVector::DotProduct(FallAcceleration, HitResult.Normal) < 0.f)
			{
				// Allow movement parallel to the wall, but not into it because that may push us up.
				const FVector Normal2D = ProjectToGravityFloor(HitResult.Normal).GetSafeNormal();
				Result = FVector::VectorPlaneProject(FallAcceleration, Normal2D);
			}
		}
	}
	else if (HitResult.bStartPenetrating)
	{
		// Allow movement out of penetration.
		return (FVector::DotProduct(Result, HitResult.Normal) > 0.f ? Result : FVector::ZeroVector);
	}

	return Result;
}


bool UCharacterMovementComponent::CheckLedgeDirection(const FVector& OldLocation, const FVector& SideStep, const FFindFloorResult& OldFloor) const
{
	const FVector SideDest = OldLocation + SideStep;
	FCollisionQueryParams CapsuleParams(SCENE_QUERY_STAT(CheckLedgeDirection), false, CharacterOwner);
	FCollisionResponseParams ResponseParam;
	InitCollisionParams(CapsuleParams, ResponseParam);
	const FCollisionShape CapsuleShape = GetPawnCapsuleCollisionShape(SHRINK_None);
	const ECollisionChannel CollisionChannel = UpdatedComponent->GetCollisionObjectType();
	FHitResult Result(1.f);
	GetWorld()->SweepSingleByChannel(Result, OldLocation, SideDest, GetWorldToGravityTransform(), CollisionChannel, CapsuleShape, CapsuleParams, ResponseParam);

	if ( !Result.bBlockingHit || IsWalkable(Result) )
	{
		if ( !Result.bBlockingHit )
		{
			GetWorld()->SweepSingleByChannel(Result, SideDest, SideDest + GetGravityDirection() * (MaxStepHeight + LedgeCheckThreshold), GetWorldToGravityTransform(), CollisionChannel, CapsuleShape, CapsuleParams, ResponseParam);
		}
		if ( (Result.Time < 1.f) && IsWalkable(Result) )
		{
			return true;
		}
	}
	return false;
}


FVector UCharacterMovementComponent::GetLedgeMove(const FVector& OldLocation, const FVector& Delta, const FFindFloorResult& OldFloor) const
{
	if (!HasValidData() || Delta.IsZero())
	{
		return FVector::ZeroVector;
	}

	// Try to base the parallel movement test on the collision with the bottom of the capsule,
	// which should inform where the perpendicular surface of the ledge is.
	bool bUseLedgeNormal = false;
	FVector LedgeNormal = FVector::ZeroVector;
	FVector LedgeMove = FVector::ZeroVector;

	if (CharacterMovementCVars::bLedgeMovementDetectEdgeNormal)
	{
		FFindFloorResult LedgeFloor = OldFloor;

		// Flat base doesn't get capsule impact data for the floor in a meaningful way, so trace using a capsule at the last valid location.
		if (bUseFlatBaseForFloorChecks)
		{
			UCharacterMovementComponent* MutableThis = const_cast<UCharacterMovementComponent*>(this);
			MutableThis->bUseFlatBaseForFloorChecks = false;
			FindFloor(OldLocation, LedgeFloor, false, NULL);
			MutableThis->bUseFlatBaseForFloorChecks = true;
		}

		if (!LedgeFloor.IsWalkableFloor())
		{
			// Fall back to current floor, though this is likely unwalkable, but may contain a usable hit.
			LedgeFloor = CurrentFloor;
		}

		// Check previous (valid) walkable floor location to get data about the ledge
		if (LedgeFloor.bBlockingHit && GetGravitySpaceZ(LedgeFloor.HitResult.Normal) > UE_KINDA_SMALL_NUMBER)
		{
			// It would be nice to use HitResult.Normal, but the quality of this normal doesn't seem as good as ImpactNormal (contains off-axis values for simple shapes).
			LedgeNormal = (LedgeFloor.HitResult.ImpactNormal ^ LedgeFloor.HitResult.Normal);
			LedgeNormal = (LedgeFloor.HitResult.ImpactNormal ^ LedgeNormal);
			LedgeNormal = ProjectToGravityFloor(LedgeNormal).GetSafeNormal();
			LedgeMove = ComputeSlideVector(Delta, 1.0f, -LedgeNormal, LedgeFloor.HitResult);
			bUseLedgeNormal = true;
		}
	}

	// In case of no good estimate of ledge, fall back to movement direction.
	if (LedgeNormal.IsZero())
	{
		LedgeMove = Delta;
		bUseLedgeNormal = false;
	}

	if (bUseLedgeNormal)
	{
		bool bLedgeHit = CheckLedgeDirection(OldLocation, LedgeMove, OldFloor);
		if (bLedgeHit)
		{
			return LedgeMove;
		}
	}
	else
	{
		FVector SideDir(Delta.Y, -1.f * Delta.X, 0.f);

		// try left
		if ( CheckLedgeDirection(OldLocation, SideDir, OldFloor) )
		{
			return SideDir;
		}

		// try right
		SideDir *= -1.f;
		if ( CheckLedgeDirection(OldLocation, SideDir, OldFloor) )
		{
			return SideDir;
		}
	}

	return FVector::ZeroVector;
}


bool UCharacterMovementComponent::CanWalkOffLedges() const
{
	if (!bCanWalkOffLedgesWhenCrouching && IsCrouching())
	{
		return false;
	}	

	return bCanWalkOffLedges;
}

bool UCharacterMovementComponent::CheckFall(const FFindFloorResult& OldFloor, const FHitResult& Hit, const FVector& Delta, const FVector& OldLocation, float remainingTime, float timeTick, int32 Iterations, bool bMustJump)
{
	if (!HasValidData())
	{
		return false;
	}

	if (bMustJump || CanWalkOffLedges())
	{
		HandleWalkingOffLedge(OldFloor.HitResult.ImpactNormal, OldFloor.HitResult.Normal, OldLocation, timeTick);
		if (IsMovingOnGround())
		{
			// If still walking, then fall. If not, assume the user set a different mode they want to keep.
			StartFalling(Iterations, remainingTime, timeTick, Delta, OldLocation);
		}
		return true;
	}
	return false;
}

void UCharacterMovementComponent::RevertMove(const FVector& OldLocation, UPrimitiveComponent* OldBase, const FVector& InOldBaseLocation, const FFindFloorResult& OldFloor, bool bFailMove)
{
	FMovementBaseInterfaceData OldMovementBaseInterfaceData(OldBase);
	RevertMove(OldLocation, &OldMovementBaseInterfaceData, InOldBaseLocation, OldFloor, bFailMove);
}

void UCharacterMovementComponent::StartFalling(int32 Iterations, float remainingTime, float timeTick, const FVector& Delta, const FVector& subLoc)
{
	// start falling 
	const float DesiredDist = Delta.Size();
	const float ActualDist = ProjectToGravityFloor(UpdatedComponent->GetComponentLocation() - subLoc).Size();
	remainingTime = (DesiredDist < UE_KINDA_SMALL_NUMBER)
					? 0.f
					: remainingTime + timeTick * (1.f - FMath::Min(1.f,ActualDist/DesiredDist));

	if ( IsMovingOnGround() )
	{
		// This is to catch cases where the first frame of PIE is executed, and the
		// level is not yet visible. In those cases, the player will fall out of the
		// world... So, don't set MOVE_Falling straight away.
		if ( !GIsEditor || (GetWorld()->HasBegunPlay() && (GetWorld()->GetTimeSeconds() >= 1.f)) )
		{
			SetMovementMode(MOVE_Falling); //default behavior if script didn't change physics
		}
		else
		{
			// Make sure that the floor check code continues processing during this delay.
			bForceNextFloorCheck = true;
		}
	}
	StartNewPhysics(remainingTime,Iterations);
}


void UCharacterMovementComponent::RevertMove(const FVector& OldLocation, FMovementBaseInterfaceData* OldMovementBaseInterfaceData, const FVector& PreviousBaseLocation, const FFindFloorResult& OldFloor, bool bFailMove)
{
	//UE_LOGF(LogCharacterMovement, Log, "RevertMove from %f %f %f to %f %f %f", CharacterOwner->Location.X, CharacterOwner->Location.Y, CharacterOwner->Location.Z, OldLocation.X, OldLocation.Y, OldLocation.Z);
	UpdatedComponent->SetWorldLocation(OldLocation, false, nullptr, GetTeleportType());
	
	//UE_LOGF(LogCharacterMovement, Log, "Now at %f %f %f", CharacterOwner->Location.X, CharacterOwner->Location.Y, CharacterOwner->Location.Z);
	bJustTeleported = false;
	// if our previous base couldn't have moved or changed in any physics-affecting way, restore it
	if (MovementBaseUtility::IsMovementBaseDataValid(OldMovementBaseInterfaceData) && 
		(!MovementBaseUtility::IsDynamicBase(OldMovementBaseInterfaceData) ||
		 (!OldMovementBaseInterfaceData->GetBodyInstanceOwner()->IsPhysicsOwnerMovable()) ||
		 (OldMovementBaseInterfaceData->GetBodyInstanceOwner()->GetPhysicsOwnerTransform().GetLocation() == PreviousBaseLocation)
		)
	   )
	{
		SetBase(OldMovementBaseInterfaceData, OldFloor.HitResult.BoneName);
		CurrentFloor = OldFloor;
	}
	else
	{
		FMovementBaseInterfaceData EmptyMovementBaseData;
		SetBase(&EmptyMovementBaseData);
	}

	if ( bFailMove )
	{
		// end movement now
		Velocity = FVector::ZeroVector;
		Acceleration = FVector::ZeroVector;
		//UE_LOGF(LogCharacterMovement, Log, "%ls FAILMOVE RevertMove", *CharacterOwner->GetName());
	}
}


FVector UCharacterMovementComponent::ComputeGroundMovementDelta(const FVector& Delta, const FHitResult& RampHit, const bool bHitFromLineTrace) const
{
	const FVector FloorNormal = RampHit.ImpactNormal;
	const FVector::FReal FloorNormalZ = GetGravitySpaceZ(FloorNormal);
	const FVector::FReal ContactNormalZ = GetGravitySpaceZ(RampHit.Normal);
	if (FloorNormalZ < (1.f - UE_KINDA_SMALL_NUMBER) && FloorNormalZ > UE_KINDA_SMALL_NUMBER && ContactNormalZ > UE_KINDA_SMALL_NUMBER && !bHitFromLineTrace && IsWalkable(RampHit))
	{
		// Compute a vector that moves parallel to the surface, by projecting the horizontal movement direction onto the ramp.
		const float FloorDotDelta = (FloorNormal | Delta);
		FVector RampMovement = Delta;
		SetGravitySpaceZ(RampMovement, -FloorDotDelta / FloorNormalZ);
		if (bMaintainHorizontalGroundVelocity)
		{
			return RampMovement;
		}
		else
		{
			return RampMovement.GetSafeNormal() * Delta.Size();
		}
	}

	return Delta;
}

void UCharacterMovementComponent::OnCharacterStuckInGeometry(const FHitResult* Hit)
{
	if (CharacterMovementCVars::StuckWarningPeriod >= 0)
	{
		const UWorld* MyWorld = GetWorld();
		const float RealTimeSeconds = MyWorld->GetRealTimeSeconds();
		if ((RealTimeSeconds - LastStuckWarningTime) >= CharacterMovementCVars::StuckWarningPeriod)
		{
			LastStuckWarningTime = RealTimeSeconds;
			if (Hit == nullptr)
			{
				UE_LOGF(LogCharacterMovement, Log, "%ls is stuck and failed to move! (%d other events since notify)", *CharacterOwner->GetName(), StuckWarningCountSinceNotify);
			}
			else
			{
				UE_LOGF(LogCharacterMovement, Log, "%ls is stuck and failed to move! Velocity: X=%3.2f Y=%3.2f Z=%3.2f Location: X=%3.2f Y=%3.2f Z=%3.2f Normal: X=%3.2f Y=%3.2f Z=%3.2f PenetrationDepth:%.3f Actor:%ls Component:%ls BoneName:%ls (%d other events since notify)",
					   *GetNameSafe(CharacterOwner),
					   Velocity.X, Velocity.Y, Velocity.Z,
					   Hit->Location.X, Hit->Location.Y, Hit->Location.Z,
					   Hit->Normal.X, Hit->Normal.Y, Hit->Normal.Z,
					   Hit->PenetrationDepth,
					   *Hit->HitObjectHandle.GetName(),
					   *GetNameSafe(Hit->GetComponent()),
					   Hit->BoneName.IsValid() ? *Hit->BoneName.ToString() : TEXT("None"),
					   StuckWarningCountSinceNotify
					   );
			}
			StuckWarningCountSinceNotify = 0;
		}
		else
		{
			StuckWarningCountSinceNotify += 1;
		}
	}

	// Don't update velocity based on our (failed) change in position this update since we're stuck.
	bJustTeleported = true;
}

void UCharacterMovementComponent::MoveAlongFloor(const FVector& InVelocity, float DeltaSeconds, FStepDownResult* OutStepDownResult)
{
	if (!CurrentFloor.IsWalkableFloor())
	{
		return;
	}

	// Move along the current floor
	const FVector Delta = ProjectToGravityFloor(InVelocity) * DeltaSeconds;
	FHitResult Hit(1.f);
	FVector RampVector = ComputeGroundMovementDelta(Delta, CurrentFloor.HitResult, CurrentFloor.bLineTrace);
	SafeMoveUpdatedComponent(RampVector, UpdatedComponent->GetComponentQuat(), true, Hit);
	float LastMoveTimeSlice = DeltaSeconds;
	
	if (Hit.bStartPenetrating)
	{
		// Allow this hit to be used as an impact we can deflect off, otherwise we do nothing the rest of the update and appear to hitch.
		HandleImpact(Hit);
		SlideAlongSurface(Delta, 1.f, Hit.Normal, Hit, true);

		if (Hit.bStartPenetrating)
		{
			OnCharacterStuckInGeometry(&Hit);
		}
	}
	else if (Hit.IsValidBlockingHit())
	{
		// We impacted something (most likely another ramp, but possibly a barrier).
		float PercentTimeApplied = Hit.Time;
		if ((Hit.Time > 0.f) && (GetGravitySpaceZ(Hit.Normal) > UE_KINDA_SMALL_NUMBER) && IsWalkable(Hit))
		{
			// Another walkable ramp.
			const float InitialPercentRemaining = 1.f - PercentTimeApplied;
			RampVector = ComputeGroundMovementDelta(Delta * InitialPercentRemaining, Hit, false);
			LastMoveTimeSlice = InitialPercentRemaining * LastMoveTimeSlice;
			SafeMoveUpdatedComponent(RampVector, UpdatedComponent->GetComponentQuat(), true, Hit);

			const float SecondHitPercent = Hit.Time * InitialPercentRemaining;
			PercentTimeApplied = FMath::Clamp(PercentTimeApplied + SecondHitPercent, 0.f, 1.f);
		}

		if (Hit.IsValidBlockingHit())
		{
			const FMovementBaseInterfaceData* MovementBaseData = GetMovementBaseInterfaceData();
			const UObject* MovementBaseObject = MovementBaseData ? GetMovementBaseInterfaceData()->PhysicsObjectOwner.Get() : nullptr;
			if (CanStepUp(Hit) || 
				(Hit.PhysicsObjectOwner.Get() == MovementBaseObject) || 
				(Hit.Component.Get() == MovementBaseObject))
			{
				// hit a barrier, try to step up
				const FVector PreStepUpLocation = UpdatedComponent->GetComponentLocation();
				if (!StepUp(GetGravityDirection(), Delta * (1.f - PercentTimeApplied), Hit, OutStepDownResult))
				{
					UE_LOGF(LogCharacterMovement, Verbose, "- StepUp (ImpactNormal %ls, Normal %ls", *Hit.ImpactNormal.ToString(), *Hit.Normal.ToString());
					HandleImpact(Hit, LastMoveTimeSlice, RampVector);
					SlideAlongSurface(Delta, 1.f - PercentTimeApplied, Hit.Normal, Hit, true);
				}
				else
				{
					UE_LOGF(LogCharacterMovement, Verbose, "+ StepUp (ImpactNormal %ls, Normal %ls", *Hit.ImpactNormal.ToString(), *Hit.Normal.ToString());
					if (!bMaintainHorizontalGroundVelocity)
					{
						// Don't recalculate velocity based on this height adjustment, if considering vertical adjustments. Only consider horizontal movement.
						bJustTeleported = true;
						const float StepUpTimeSlice = (1.f - PercentTimeApplied) * DeltaSeconds;
						if (!HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() && StepUpTimeSlice >= UE_KINDA_SMALL_NUMBER)
						{
							Velocity = (UpdatedComponent->GetComponentLocation() - PreStepUpLocation) / StepUpTimeSlice;
							Velocity = ProjectToGravityFloor(Velocity);
						}
					}
				}
			}
			else if ( Hit.Component.IsValid() && !Hit.Component.Get()->CanCharacterStepUp(CharacterOwner) )
			{
				HandleImpact(Hit, LastMoveTimeSlice, RampVector);
				SlideAlongSurface(Delta, 1.f - PercentTimeApplied, Hit.Normal, Hit, true);
			}
		}
	}
}


void UCharacterMovementComponent::MaintainHorizontalGroundVelocity()
{
	if (GetGravitySpaceZ(Velocity) != 0.f)
	{
		if (bMaintainHorizontalGroundVelocity)
		{
			// Ramp movement already maintained the velocity, so we just want to remove the vertical component.
			Velocity = ProjectToGravityFloor(Velocity);
		}
		else
		{
			// Rescale velocity to be horizontal but maintain magnitude of last update.
			Velocity = ProjectToGravityFloor(Velocity).GetSafeNormal() * Velocity.Size();
		}
	}
}


void UCharacterMovementComponent::PhysWalking(float deltaTime, int32 Iterations)
{
	SCOPE_CYCLE_COUNTER(STAT_CharPhysWalking);

	if (deltaTime < MIN_TICK_TIME)
	{
		return;
	}

	if (!CharacterOwner || (!CharacterOwner->GetController() && !bRunPhysicsWithNoController && !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() && (CharacterOwner->GetLocalRole() != ROLE_SimulatedProxy)))
	{
		Acceleration = FVector::ZeroVector;
		Velocity = FVector::ZeroVector;
		return;
	}

	if (!UpdatedComponent->IsQueryCollisionEnabled())
	{
		SetMovementMode(MOVE_Walking);
		return;
	}

	devCode(ensureMsgf(!Velocity.ContainsNaN(), TEXT("PhysWalking: Velocity contains NaN before Iteration (%s)\n%s"), *GetPathNameSafe(this), *Velocity.ToString()));
	
	bJustTeleported = false;
	bool bCheckedFall = false;
	bool bTriedLedgeMove = false;
	float remainingTime = deltaTime;

	const EMovementMode StartingMovementMode = MovementMode;
	const uint8 StartingCustomMovementMode = CustomMovementMode;

	// Perform the move
	while ( (remainingTime >= MIN_TICK_TIME) && (Iterations < MaxSimulationIterations) && CharacterOwner && (CharacterOwner->GetController() || bRunPhysicsWithNoController || HasAnimRootMotion() || CurrentRootMotion.HasOverrideVelocity() || (CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy)) )
	{
		Iterations++;
		bJustTeleported = false;
		const float timeTick = GetSimulationTimeStep(remainingTime, Iterations);
		remainingTime -= timeTick;

#if UE_WITH_REMOTE_OBJECT_HANDLE
		//Scale down impact force if CharacterMoveComponent is taking multiple substeps.
		const float LastFrameDt = GetWorld()->GetDeltaSeconds();
		PhysicsForceSubsteppingFactor = timeTick / LastFrameDt;
#endif
		
		// Save current values
		FMovementBaseInterfaceData* OldMovementBaseInterfaceData = GetMovementBaseInterfaceData_Mutable();
		const FVector PreviousBaseLocation = OldMovementBaseInterfaceData && OldMovementBaseInterfaceData->IsValid() ? OldMovementBaseInterfaceData->GetBodyInstanceOwner()->GetPhysicsOwnerTransform().GetLocation() : FVector::ZeroVector;
		const FVector OldLocation = UpdatedComponent->GetComponentLocation();
		const FFindFloorResult OldFloor = CurrentFloor;

		RestorePreAdditiveRootMotionVelocity();

		// Ensure velocity is horizontal.
		MaintainHorizontalGroundVelocity();
		const FVector OldVelocity = Velocity;
		Acceleration = FVector::VectorPlaneProject(Acceleration, -GravityDirection);

		// Apply acceleration
		const bool bSkipForLedgeMove = bTriedLedgeMove && CharacterMovementCVars::bLedgeMovementApplyDirectMove;
		if( !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() && !bSkipForLedgeMove )
		{
			CalcVelocity(timeTick, GroundFriction, false, GetMaxBrakingDeceleration());
			devCode(ensureMsgf(!Velocity.ContainsNaN(), TEXT("PhysWalking: Velocity contains NaN after CalcVelocity (%s)\n%s"), *GetPathNameSafe(this), *Velocity.ToString()));
		}
		
		ApplyRootMotionToVelocity(timeTick);
		devCode(ensureMsgf(!Velocity.ContainsNaN(), TEXT("PhysWalking: Velocity contains NaN after Root Motion application (%s)\n%s"), *GetPathNameSafe(this), *Velocity.ToString()));

		if (MovementMode != StartingMovementMode || CustomMovementMode != StartingCustomMovementMode)
		{
			// Root motion could have taken us out of our current mode
			// No movement has taken place this movement tick so we pass on full time/past iteration count
			StartNewPhysics(remainingTime+timeTick, Iterations-1);
			return;
		}

		// Compute move parameters
		const FVector MoveVelocity = Velocity;
		const FVector Delta = timeTick * MoveVelocity;
		const bool bZeroDelta = Delta.IsNearlyZero();
		FStepDownResult StepDownResult;

		if ( bZeroDelta )
		{
			remainingTime = 0.f;
		}
		else
		{
			// try to move forward
			MoveAlongFloor(MoveVelocity, timeTick, &StepDownResult);

			if (IsSwimming()) //just entered water
			{
				StartSwimming(OldLocation, OldVelocity, timeTick, remainingTime, Iterations);
				return;
			}
			else if (MovementMode != StartingMovementMode || CustomMovementMode != StartingCustomMovementMode)
			{
				// pawn ended up in a different mode, probably due to the step-up-and-over flow
				// let's refund the estimated unused time (if any) and keep moving in the new mode
				const float DesiredDist = Delta.Size();
				if (DesiredDist > UE_KINDA_SMALL_NUMBER)
				{
					const float ActualDist = ProjectToGravityFloor(UpdatedComponent->GetComponentLocation() - OldLocation).Size();
					remainingTime += timeTick * (1.f - FMath::Min(1.f,ActualDist/DesiredDist));
				}
				StartNewPhysics(remainingTime,Iterations);
				return;
			}
		}

		// Update floor.
		// StepUp might have already done it for us.
		if (StepDownResult.bComputedFloor)
		{
			CurrentFloor = StepDownResult.FloorResult;
		}
		else
		{
			FindFloor(UpdatedComponent->GetComponentLocation(), CurrentFloor, bZeroDelta, NULL);
		}

		// check for ledges here
		const bool bCheckLedges = !CanWalkOffLedges();
		if ( bCheckLedges && !CurrentFloor.IsWalkableFloor() )
		{
			// calculate possible alternate movement
			const FVector NewDelta = bTriedLedgeMove ? FVector::ZeroVector : GetLedgeMove(OldLocation, Delta, OldFloor);
			if ( !NewDelta.IsZero() )
			{
				// first revert this move
				RevertMove(OldLocation, OldMovementBaseInterfaceData, PreviousBaseLocation, OldFloor, false);

				// avoid repeated ledge moves if the first one fails
				bTriedLedgeMove = true;

				// Try new movement direction
				Velocity = NewDelta/timeTick;
				remainingTime += timeTick;
				Iterations--;
				continue;
			}
			else
			{
				// see if it is OK to jump
				// @todo collision : only thing that can be problem is that oldbase has world collision on
				const bool bOldMovementBaseValid = OldMovementBaseInterfaceData && OldMovementBaseInterfaceData->IsValid();
				bool bMustJump = bZeroDelta || (!bOldMovementBaseValid || (!CollisionEnabledHasQuery(OldMovementBaseInterfaceData->GetBodyInstanceOwner()->GetCollisionEnabled()) && MovementBaseUtility::IsDynamicBase(OldMovementBaseInterfaceData)));
				if ( (bMustJump || !bCheckedFall) && CheckFall(OldFloor, CurrentFloor.HitResult, Delta, OldLocation, remainingTime, timeTick, Iterations, bMustJump) )
				{
					return;
				}
				bCheckedFall = true;

				// revert this move
				RevertMove(OldLocation, OldMovementBaseInterfaceData, PreviousBaseLocation, OldFloor, true);
				remainingTime = 0.f;
				break;
			}
		}
		else
		{
			// Validate the floor check
			if (CurrentFloor.IsWalkableFloor())
			{
				if (ShouldCatchAir(OldFloor, CurrentFloor))
				{
					HandleWalkingOffLedge(OldFloor.HitResult.ImpactNormal, OldFloor.HitResult.Normal, OldLocation, timeTick);
					if (IsMovingOnGround())
					{
						// If still walking, then fall. If not, assume the user set a different mode they want to keep.
						StartFalling(Iterations, remainingTime, timeTick, Delta, OldLocation);
					}
					return;
				}

				AdjustFloorHeight();
				SetBaseFromFloor(CurrentFloor);
			}
			else if (CurrentFloor.HitResult.bStartPenetrating && remainingTime <= 0.f)
			{
				// The floor check failed because it started in penetration
				// We do not want to try to move downward because the downward sweep failed, rather we'd like to try to pop out of the floor.
				FHitResult Hit(CurrentFloor.HitResult);
				Hit.TraceEnd = Hit.TraceStart + MAX_FLOOR_DIST * -GetGravityDirection();
				const FVector RequestedAdjustment = GetPenetrationAdjustment(Hit);
				ResolvePenetration(RequestedAdjustment, Hit, UpdatedComponent->GetComponentQuat());
				bForceNextFloorCheck = true;
			}

			// check if just entered water
			if ( IsSwimming() )
			{
				StartSwimming(OldLocation, Velocity, timeTick, remainingTime, Iterations);
				return;
			}

			// See if we need to start falling.
			if (!CurrentFloor.IsWalkableFloor() && !CurrentFloor.HitResult.bStartPenetrating)
			{
				const bool bOldMovementBaseValid = OldMovementBaseInterfaceData && OldMovementBaseInterfaceData->IsValid();
				const bool bMustJump = bJustTeleported || bZeroDelta || (!bOldMovementBaseValid || (!CollisionEnabledHasQuery(OldMovementBaseInterfaceData->GetBodyInstanceOwner()->GetCollisionEnabled()) && MovementBaseUtility::IsDynamicBase(OldMovementBaseInterfaceData)));
				if ((bMustJump || !bCheckedFall) && CheckFall(OldFloor, CurrentFloor.HitResult, Delta, OldLocation, remainingTime, timeTick, Iterations, bMustJump) )
				{
					return;
				}
				bCheckedFall = true;
			}
		}


		// Allow overlap events and such to change physics state and velocity
		if (IsMovingOnGround())
		{
			// Make velocity reflect actual move
			if( !bJustTeleported && !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity() && timeTick >= MIN_TICK_TIME)
			{
				// TODO-RootMotionSource: Allow this to happen during partial override Velocity, but only set allowed axes?
				Velocity = (UpdatedComponent->GetComponentLocation() - OldLocation) / timeTick;
				MaintainHorizontalGroundVelocity();
			}
		}

		// If we didn't move at all this iteration then abort (since future iterations will also be stuck).
		if (UpdatedComponent->GetComponentLocation() == OldLocation)
		{
			remainingTime = 0.f;
			break;
		}	
	}

	if (IsMovingOnGround())
	{
		MaintainHorizontalGroundVelocity();
	}
}

void UCharacterMovementComponent::PhysNavWalking(float deltaTime, int32 Iterations)
{
	SCOPE_CYCLE_COUNTER(STAT_CharPhysNavWalking);

	if (deltaTime < MIN_TICK_TIME)
	{
		return;
	}

	if ((!CharacterOwner || !CharacterOwner->GetController()) && !bRunPhysicsWithNoController && !HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity())
	{
		Acceleration = FVector::ZeroVector;
		Velocity = FVector::ZeroVector;
		return;
	}

	const EMovementMode StartingMovementMode = MovementMode;
	const uint8 StartingCustomMovementMode = CustomMovementMode;

	RestorePreAdditiveRootMotionVelocity();

	// Ensure velocity is horizontal.
	MaintainHorizontalGroundVelocity();
	devCode(ensureMsgf(!Velocity.ContainsNaN(), TEXT("PhysNavWalking: Velocity contains NaN before CalcVelocity (%s)\n%s"), *GetPathNameSafe(this), *Velocity.ToString()));

	//bound acceleration
	Acceleration = ProjectToGravityFloor(Acceleration);
	if (!HasAnimRootMotion() && !CurrentRootMotion.HasOverrideVelocity())
	{
		CalcVelocity(deltaTime, GroundFriction, false, GetMaxBrakingDeceleration());
		devCode(ensureMsgf(!Velocity.ContainsNaN(), TEXT("PhysNavWalking: Velocity contains NaN after CalcVelocity (%s)\n%s"), *GetPathNameSafe(this), *Velocity.ToString()));
	}

	ApplyRootMotionToVelocity(deltaTime);

	if (MovementMode != StartingMovementMode || CustomMovementMode != StartingCustomMovementMode)
	{
		// Root motion could have taken us out of our current mode
		StartNewPhysics(deltaTime, Iterations);
		return;
	}

	Iterations++;

	const FVector DesiredMove = ProjectToGravityFloor(Velocity);

	const FVector OldLocation = GetActorFeetLocation();
	const FVector DeltaMove = DesiredMove * deltaTime;
	const bool bDeltaMoveNearlyZero = DeltaMove.IsNearlyZero();

	FVector AdjustedDest = OldLocation + DeltaMove;
	FNavLocation DestNavLocation;

	bool bSameNavLocation = false;
	if (CachedNavLocation.NodeRef != INVALID_NAVNODEREF)
	{
		if (bProjectNavMeshWalking)
		{
			const float DistSq2D = ProjectToGravityFloor(OldLocation - CachedNavLocation.Location).SizeSquared();
			const float DistZ = FMath::Abs(GetGravitySpaceZ(OldLocation - CachedNavLocation.Location));

			const float TotalCapsuleHeight = CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleHalfHeight() * 2.0f;
			const float ProjectionScale = (GetGravitySpaceZ(OldLocation) > GetGravitySpaceZ(CachedNavLocation.Location)) ? NavMeshProjectionHeightScaleUp : NavMeshProjectionHeightScaleDown;
			const float DistZThr = TotalCapsuleHeight * FMath::Max(0.f, ProjectionScale);

			bSameNavLocation = (DistSq2D <= UE_KINDA_SMALL_NUMBER) && (DistZ < DistZThr);
		}
		else
		{
			bSameNavLocation = CachedNavLocation.Location.Equals(OldLocation);
		}

		if (bDeltaMoveNearlyZero && bSameNavLocation)
		{
			if (const INavigationDataInterface* NavData = GetNavData())
			{
				if (!NavData->IsNodeRefValid(CachedNavLocation.NodeRef))
				{
					CachedNavLocation.NodeRef = INVALID_NAVNODEREF;
					bSameNavLocation = false;
				}
			}
		}
	}

	if (bDeltaMoveNearlyZero && bSameNavLocation)
	{
		DestNavLocation = CachedNavLocation;
		UE_LOGF(LogNavMeshMovement, VeryVerbose, "%ls using cached navmesh location! (bProjectNavMeshWalking = %d)", *GetNameSafe(CharacterOwner), bProjectNavMeshWalking);
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_CharNavProjectPoint);

		// Start the trace from the Z location of the last valid trace.
		// Otherwise if we are projecting our location to the underlying geometry and it's far above or below the navmesh,
		// we'll follow that geometry's plane out of range of valid navigation.
		if (bSameNavLocation && bProjectNavMeshWalking)
		{
			SetGravitySpaceZ(AdjustedDest, GetGravitySpaceZ(CachedNavLocation.Location));
		}

		bool bFoundPointOnNavMesh = false;
		if (bSlideAlongNavMeshEdge)
		{
			if (const INavigationDataInterface* NavDataInterface = GetNavData())
			{
				const IPathFollowingAgentInterface* PathFollowingAgent = GetPathFollowingAgent();
				const bool bIsOnNavLink = PathFollowingAgent && PathFollowingAgent->IsFollowingNavLink();

				if (!bIsOnNavLink)
				{
					FNavLocation StartingNavFloorLocation;
					bool bHasValidCachedNavLocation = NavDataInterface->IsNodeRefValid(CachedNavLocation.NodeRef);

					// If we don't have a valid CachedNavLocation lets try finding the NavFloor where we're currently at and use that
					if (!bHasValidCachedNavLocation)
					{
						bHasValidCachedNavLocation = FindNavFloor(OldLocation, OUT StartingNavFloorLocation);
					}
					else
					{
						StartingNavFloorLocation = CachedNavLocation;
					}

					if (bHasValidCachedNavLocation)
					{
						bFoundPointOnNavMesh = NavDataInterface->FindMoveAlongSurface(StartingNavFloorLocation, AdjustedDest, OUT DestNavLocation);

						if (bFoundPointOnNavMesh)
						{
							AdjustedDest = ProjectToGravityFloor(DestNavLocation.Location) + GetGravitySpaceComponentZ(AdjustedDest);
						}
					}
				}
				else
				{
					bFoundPointOnNavMesh = FindNavFloor(AdjustedDest, DestNavLocation);
				}
			}
		}
		else
		{
			bFoundPointOnNavMesh = FindNavFloor(AdjustedDest, DestNavLocation);
		}
		
		if (!bFoundPointOnNavMesh)
		{
			SetMovementMode(MOVE_Walking);
			return;
		}

		CachedNavLocation = DestNavLocation;
	}

	if (DestNavLocation.NodeRef != INVALID_NAVNODEREF)
	{
		FVector NewLocation = ProjectToGravityFloor(AdjustedDest) + GetGravitySpaceComponentZ(DestNavLocation.Location);
		if (bProjectNavMeshWalking)
		{
			SCOPE_CYCLE_COUNTER(STAT_CharNavProjectLocation);
			const float TotalCapsuleHeight = CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleHalfHeight() * 2.0f;
			const float UpOffset = TotalCapsuleHeight * FMath::Max(0.f, NavMeshProjectionHeightScaleUp);
			const float DownOffset = TotalCapsuleHeight * FMath::Max(0.f, NavMeshProjectionHeightScaleDown);
			NewLocation = ProjectLocationFromNavMesh(deltaTime, OldLocation, NewLocation, UpOffset, DownOffset);
		}

		FVector AdjustedDelta = NewLocation - OldLocation;

		if (!AdjustedDelta.IsNearlyZero())
		{
			FHitResult HitResult;
			SafeMoveUpdatedComponent(AdjustedDelta, UpdatedComponent->GetComponentQuat(), bSweepWhileNavWalking, HitResult);
		}

		// Update velocity to reflect actual move
		if (!bJustTeleported && !HasAnimRootMotion() && !CurrentRootMotion.HasVelocity())
		{
			Velocity = (GetActorFeetLocation() - OldLocation) / deltaTime;
			MaintainHorizontalGroundVelocity();
		}

		bJustTeleported = false;
	}
	else
	{
		StartFalling(Iterations, deltaTime, deltaTime, DeltaMove, OldLocation);
	}
}

bool UCharacterMovementComponent::FindNavFloor(const FVector& TestLocation, FNavLocation& NavFloorLocation) const
{
	const INavigationDataInterface* NavData = GetNavData();
	if (NavData == nullptr || CharacterOwner == nullptr)
	{
		return false;
	}

	const FNavAgentProperties& AgentProps = CharacterOwner->GetNavAgentPropertiesRef();
	const float SearchRadius = AgentProps.AgentRadius * 2.0f;
	const float SearchHeight = AgentProps.AgentHeight * AgentProps.NavWalkingSearchHeightScale;

	return NavData->ProjectPoint(TestLocation, NavFloorLocation, FVector(SearchRadius, SearchRadius, SearchHeight));
}

FVector UCharacterMovementComponent::ProjectLocationFromNavMesh(float DeltaSeconds, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, float UpOffset, float DownOffset)
{
	SCOPE_CYCLE_COUNTER(STAT_CharNavProjectLocation);

	FVector NewLocation = TargetNavLocation;

	const float ZOffset = -(DownOffset + UpOffset);
	if (ZOffset > -UE_SMALL_NUMBER)
	{
		return NewLocation;
	}

	const FVector TraceStart = TargetNavLocation + UpOffset * -GetGravityDirection();
	const FVector TraceEnd   = TargetNavLocation + DownOffset * GetGravityDirection();

	// We can skip this trace if we are checking at the same location as the last trace (ie, we haven't moved).
	const bool bCachedLocationStillValid = (CachedProjectedNavMeshHitResult.bBlockingHit &&
											CachedProjectedNavMeshHitResult.TraceStart == TraceStart &&
											CachedProjectedNavMeshHitResult.TraceEnd == TraceEnd);

	NavMeshProjectionTimer -= DeltaSeconds;
	if (NavMeshProjectionTimer <= 0.0f)
	{
		if (!bCachedLocationStillValid || bAlwaysCheckFloor)
		{
			UE_LOGF(LogNavMeshMovement, VeryVerbose, "ProjectLocationFromNavMesh(): %ls interval: %.3f velocity: %ls", *GetNameSafe(CharacterOwner), NavMeshProjectionInterval, *Velocity.ToString());

			FHitResult HitResult;
			FindBestNavMeshLocation(TraceStart, TraceEnd, CurrentFeetLocation, TargetNavLocation, HitResult);

			// discard result if we were already inside something			
			if (HitResult.bStartPenetrating || !HitResult.bBlockingHit)
			{
				CachedProjectedNavMeshHitResult.Reset();
			}
			else
			{
				CachedProjectedNavMeshHitResult = HitResult;
			}
		}
		else
		{
			UE_LOGF(LogNavMeshMovement, VeryVerbose, "ProjectLocationFromNavMesh(): %ls interval: %.3f velocity: %ls [SKIP TRACE]", *GetNameSafe(CharacterOwner), NavMeshProjectionInterval, *Velocity.ToString());
		}

		// Wrap around to maintain same relative offset to tick time changes.
		// Prevents large framerate spikes from aligning multiple characters to the same frame (if they start staggered, they will now remain staggered).
		float ModTime = 0.f;
		if (NavMeshProjectionInterval > UE_SMALL_NUMBER)
		{
			ModTime = FMath::Fmod(-NavMeshProjectionTimer, NavMeshProjectionInterval);
		}

		NavMeshProjectionTimer = NavMeshProjectionInterval - ModTime;
	}
	
	// Project to last plane we found.
	if (CachedProjectedNavMeshHitResult.bBlockingHit)
	{
		if (bCachedLocationStillValid && FMath::IsNearlyEqual(GetGravitySpaceZ(CurrentFeetLocation), GetGravitySpaceZ(CachedProjectedNavMeshHitResult.ImpactPoint), (FVector::FReal)0.01f))
		{
			// Already at destination.
			SetGravitySpaceZ(NewLocation, GetGravitySpaceZ(CurrentFeetLocation));
		}
		else
		{
			FVector::FReal ProjectedZ;
			if (HasCustomGravity())
			{
				const FVector ProjectedPoint = FMath::LinePlaneIntersection(TraceStart, TraceEnd, CachedProjectedNavMeshHitResult.ImpactPoint, CachedProjectedNavMeshHitResult.Normal);
				ProjectedZ = GetGravitySpaceZ(ProjectedPoint);
			}
			else
			{
				// Optimized assuming we only care about Z coordinate of result.
				const FVector& PlaneOrigin = CachedProjectedNavMeshHitResult.ImpactPoint;
				const FVector& PlaneNormal = CachedProjectedNavMeshHitResult.Normal;
				ProjectedZ = TraceStart.Z + ZOffset * (((PlaneOrigin - TraceStart)|PlaneNormal) / (ZOffset * PlaneNormal.Z));
			}
			
			// Limit to not be too far above or below NavMesh location
			const FVector::FReal TraceStartZ = GetGravitySpaceZ(TraceStart);
			const FVector::FReal TraceEndZ = GetGravitySpaceZ(TraceEnd);
			const FVector::FReal TraceMinZ = FMath::Min(TraceStartZ, TraceEndZ);
			const FVector::FReal TraceMaxZ = FMath::Max(TraceStartZ, TraceEndZ);
			ProjectedZ = FMath::Clamp(ProjectedZ, TraceMinZ, TraceMaxZ);

			// Interp for smoother updates (less "pop" when trace hits something new). 0 interp speed is instant.
			const FVector::FReal InterpSpeed = FMath::Max<FVector::FReal>(0.f, NavMeshProjectionInterpSpeed);
			ProjectedZ = FMath::FInterpTo(GetGravitySpaceZ(CurrentFeetLocation), ProjectedZ, (FVector::FReal)DeltaSeconds, InterpSpeed);
			ProjectedZ = FMath::Clamp(ProjectedZ, TraceMinZ, TraceMaxZ);
			
			// Final result
			SetGravitySpaceZ(NewLocation, ProjectedZ);
		}
	}

	return NewLocation;
}

void UCharacterMovementComponent::FindBestNavMeshLocation(const FVector& TraceStart, const FVector& TraceEnd, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, FHitResult& OutHitResult) const
{
	// raycast to underlying mesh to allow us to more closely follow geometry
	// we use static objects here as a best approximation to accept only objects that
	// influence navmesh generation
	FCollisionQueryParams Params(SCENE_QUERY_STAT(ProjectLocation), false);

	// blocked by world static and optionally world dynamic
	FCollisionResponseParams ResponseParams(ECR_Ignore);
	ResponseParams.CollisionResponse.SetResponse(ECC_WorldStatic, ECR_Overlap);
	ResponseParams.CollisionResponse.SetResponse(ECC_WorldDynamic, bProjectNavMeshOnBothWorldChannels ? ECR_Overlap : ECR_Ignore);

	TArray<FHitResult> MultiTraceHits;
	GetWorld()->LineTraceMultiByChannel(MultiTraceHits, TraceStart, TraceEnd, ECC_WorldStatic, Params, ResponseParams);

	struct FCompareFHitResultNavMeshTrace
	{
		explicit FCompareFHitResultNavMeshTrace(const FVector& inSourceLocation) : SourceLocation(inSourceLocation)
		{
		}

		FORCEINLINE bool operator()(const FHitResult& A, const FHitResult& B) const
		{
			const float ADistSqr = (SourceLocation - A.ImpactPoint).SizeSquared();
			const float BDistSqr = (SourceLocation - B.ImpactPoint).SizeSquared();

			return (ADistSqr < BDistSqr);
		}

		const FVector& SourceLocation;
	};

	struct FRemoveNotBlockingResponseNavMeshTrace
	{
		FRemoveNotBlockingResponseNavMeshTrace(bool bInCheckOnlyWorldStatic) : bCheckOnlyWorldStatic(bInCheckOnlyWorldStatic) {}

		FORCEINLINE bool operator()(const FHitResult& TestHit) const
		{
			UPrimitiveComponent* PrimComp = TestHit.GetComponent();
			const bool bBlockOnWorldStatic = PrimComp && (PrimComp->GetCollisionResponseToChannel(ECC_WorldStatic) == ECR_Block);
			const bool bBlockOnWorldDynamic = PrimComp && (PrimComp->GetCollisionResponseToChannel(ECC_WorldDynamic) == ECR_Block);

			return !bBlockOnWorldStatic && (!bBlockOnWorldDynamic || bCheckOnlyWorldStatic);
		}

		bool bCheckOnlyWorldStatic;
	};

	MultiTraceHits.RemoveAllSwap(FRemoveNotBlockingResponseNavMeshTrace(!bProjectNavMeshOnBothWorldChannels), EAllowShrinking::No);
	if (MultiTraceHits.Num() > 0)
	{
		// Sort the hits by the closest to our origin.
		MultiTraceHits.Sort(FCompareFHitResultNavMeshTrace(TargetNavLocation));

		// Cache the closest hit and treat it as a blocking hit (we used an overlap to get all the world static hits so we could sort them ourselves)
		OutHitResult = MultiTraceHits[0];
		OutHitResult.bBlockingHit = true;
	}
}

const INavigationDataInterface* UCharacterMovementComponent::GetNavData() const
{
	const UWorld* World = GetWorld();
	if (World == nullptr || World->GetNavigationSystem() == nullptr 
		|| !HasValidData()
		|| CharacterOwner == nullptr)
	{
		return nullptr;
	}

	const INavigationDataInterface* NavData = FNavigationSystem::GetNavDataForActor(*CharacterOwner);

	return NavData;
}


void UCharacterMovementComponent::PhysCustom(float deltaTime, int32 Iterations)
{
	if (CharacterOwner)
	{
		CharacterOwner->K2_UpdateCustomMovement(deltaTime);
	}
}


bool UCharacterMovementComponent::ShouldCatchAir(const FFindFloorResult& OldFloor, const FFindFloorResult& NewFloor)
{
	return false;
}

void UCharacterMovementComponent::HandleWalkingOffLedge(const FVector& PreviousFloorImpactNormal, const FVector& PreviousFloorContactNormal, const FVector& PreviousLocation, float TimeDelta)
{
	if (CharacterOwner)
	{
		CharacterOwner->OnWalkingOffLedge(PreviousFloorImpactNormal, PreviousFloorContactNormal, PreviousLocation, TimeDelta);
	}
}

void UCharacterMovementComponent::AdjustFloorHeight()
{
	SCOPE_CYCLE_COUNTER(STAT_CharAdjustFloorHeight);

	// If we have a floor check that hasn't hit anything, don't adjust height.
	if (!CurrentFloor.IsWalkableFloor())
	{
		return;
	}

	float OldFloorDist = CurrentFloor.FloorDist;
	if (CurrentFloor.bLineTrace)
	{
		if (OldFloorDist < MIN_FLOOR_DIST && CurrentFloor.LineDist >= MIN_FLOOR_DIST)
		{
			// This would cause us to scale unwalkable walls
			UE_LOGF(LogCharacterMovement, VeryVerbose, "Adjust floor height aborting due to line trace with small floor distance (line: %.2f, sweep: %.2f)", CurrentFloor.LineDist, CurrentFloor.FloorDist);
			return;
		}
		else
		{
			// Falling back to a line trace means the sweep was unwalkable (or in penetration). Use the line distance for the vertical adjustment.
			OldFloorDist = CurrentFloor.LineDist;
		}
	}

	// Move up or down to maintain floor height.
	if (OldFloorDist < MIN_FLOOR_DIST || OldFloorDist > MAX_FLOOR_DIST)
	{
		FHitResult AdjustHit(1.f);
		const double InitialZ = GetGravitySpaceZ(UpdatedComponent->GetComponentLocation());
		const float AvgFloorDist = (MIN_FLOOR_DIST + MAX_FLOOR_DIST) * 0.5f;
		const float MoveDist = AvgFloorDist - OldFloorDist;
		SafeMoveUpdatedComponent(MoveDist * -GetGravityDirection(), UpdatedComponent->GetComponentQuat(), true, AdjustHit );
		UE_LOGF(LogCharacterMovement, VeryVerbose, "Adjust floor height %.3f (Hit = %d)", MoveDist, AdjustHit.bBlockingHit);

		if (!AdjustHit.IsValidBlockingHit())
		{
			CurrentFloor.FloorDist += MoveDist;
		}
		else if (MoveDist > 0.f)
		{
			const double CurrentZ = GetGravitySpaceZ(UpdatedComponent->GetComponentLocation());
			CurrentFloor.FloorDist += CurrentZ - InitialZ;
		}
		else
		{
			checkSlow(MoveDist < 0.f);
			CurrentFloor.FloorDist = GetGravitySpaceZ(UpdatedComponent->GetComponentLocation() - AdjustHit.Location);
			if (IsWalkable(AdjustHit))
			{
				CurrentFloor.SetFromSweep(AdjustHit, CurrentFloor.FloorDist, true);
			}
		}

		// Don't recalculate velocity based on this height adjustment, if considering vertical adjustments.
		// Also avoid it if we moved out of penetration
		bJustTeleported |= !bMaintainHorizontalGroundVelocity || (OldFloorDist < 0.f);
		
		// If something caused us to adjust our height (especially a depentration) we should ensure another check next frame or we will keep a stale result.
		if (CharacterOwner && CharacterOwner->GetLocalRole() != ROLE_SimulatedProxy)
		{
			bForceNextFloorCheck = true;
		}
	}
}


void UCharacterMovementComponent::StopActiveMovement() 
{ 
	Super::StopActiveMovement();

	Acceleration = FVector::ZeroVector; 
	bHasRequestedVelocity = false;
	RequestedVelocity = FVector::ZeroVector;
	LastUpdateRequestedVelocity = FVector::ZeroVector;
}

void UCharacterMovementComponent::ProcessLanded(const FHitResult& Hit, float remainingTime, int32 Iterations)
{
	SCOPE_CYCLE_COUNTER(STAT_CharProcessLanded);

	if( CharacterOwner && CharacterOwner->ShouldNotifyLanded(Hit) )
	{
		CharacterOwner->Landed(Hit);
	}
	if( IsFalling() )
	{
		if (GroundMovementMode == MOVE_NavWalking)
		{
			// verify navmesh projection and current floor
			// otherwise movement will be stuck in infinite loop:
			// navwalking -> (no navmesh) -> falling -> (standing on something) -> navwalking -> ....

			const FVector TestLocation = GetActorFeetLocation();
			FNavLocation NavLocation;

			const bool bHasNavigationData = FindNavFloor(TestLocation, NavLocation);
			if (!bHasNavigationData || NavLocation.NodeRef == INVALID_NAVNODEREF)
			{
				GroundMovementMode = MOVE_Walking;
				UE_LOGF(LogNavMeshMovement, Verbose, "ProcessLanded(): %ls tried to go to NavWalking but couldn't find NavMesh! Using Walking instead.", *GetNameSafe(CharacterOwner));
			}
		}

		SetPostLandedPhysics(Hit);
	}
	
	IPathFollowingAgentInterface* PFAgent = GetPathFollowingAgent();
	if (PFAgent)
	{
		PFAgent->OnLanded();
	}

	StartNewPhysics(remainingTime, Iterations);
}

void UCharacterMovementComponent::SetPostLandedPhysics(const FHitResult& Hit)
{
	if( CharacterOwner )
	{
		if (CanEverSwim() && IsInWater())
		{
			SetMovementMode(MOVE_Swimming);
		}
		else
		{
			const FVector PreImpactAccel = Acceleration + (IsFalling() ? -GetGravityDirection() * GetGravityZ() : FVector::ZeroVector);
			const FVector PreImpactVelocity = Velocity;

			if (DefaultLandMovementMode == MOVE_Walking ||
				DefaultLandMovementMode == MOVE_NavWalking ||
				DefaultLandMovementMode == MOVE_Falling)
			{
				SetMovementMode(GroundMovementMode);
			}
			else
			{
				SetDefaultMovementMode();
			}
			
			ApplyImpactPhysicsForces(Hit, PreImpactAccel, PreImpactVelocity);
		}
	}
}

void UCharacterMovementComponent::ControlledCharacterMove(const FVector& InputVector, float DeltaSeconds)
{
	{
		SCOPE_CYCLE_COUNTER(STAT_CharUpdateAcceleration);

		// We need to check the jump state before adjusting input acceleration, to minimize latency
		// and to make sure acceleration respects our potentially new falling state.
		CharacterOwner->CheckJumpInput(DeltaSeconds);

		// apply input to acceleration
		Acceleration = ScaleInputAcceleration(ConstrainInputAcceleration(InputVector));
		AnalogInputModifier = ComputeAnalogInputModifier();
	}

	if (CharacterOwner->GetLocalRole() == ROLE_Authority)
	{
		PerformMovement(DeltaSeconds);
	}
	else if (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy && IsNetMode(NM_Client))
	{
		ReplicateMoveToServer(DeltaSeconds, Acceleration);
	}
}

void UCharacterMovementComponent::SetNavWalkingPhysics(bool bEnable)
{
	if (UpdatedPrimitive)
	{
		if (bEnable)
		{
			UpdatedPrimitive->SetCollisionResponseToChannel(ECC_WorldStatic, ECR_Ignore);
			UpdatedPrimitive->SetCollisionResponseToChannel(ECC_WorldDynamic, ECR_Ignore);
			CachedProjectedNavMeshHitResult.Reset();

			// Stagger timed updates so many different characters spawned at the same time don't update on the same frame.
			// Initially we want an immediate update though, so set time to a negative randomized range.
			NavMeshProjectionTimer = (NavMeshProjectionInterval > 0.f) ? FMath::FRandRange(-NavMeshProjectionInterval, 0.f) : 0.f;
		}
		else
		{
			UPrimitiveComponent* DefaultCapsule = nullptr;
			if (CharacterOwner && CharacterOwner->GetCapsuleComponent() == UpdatedComponent)
			{
				ACharacter* DefaultCharacter = CharacterOwner->GetClass()->GetDefaultObject<ACharacter>();
				DefaultCapsule = DefaultCharacter ? DefaultCharacter->GetCapsuleComponent() : nullptr;
			}

			if (DefaultCapsule)
			{
				UpdatedPrimitive->SetCollisionResponseToChannel(ECC_WorldStatic, DefaultCapsule->GetCollisionResponseToChannel(ECC_WorldStatic));
				UpdatedPrimitive->SetCollisionResponseToChannel(ECC_WorldDynamic, DefaultCapsule->GetCollisionResponseToChannel(ECC_WorldDynamic));
			}
			else
			{
				UE_LOGF(LogCharacterMovement, Warning, "Can't revert NavWalking collision settings for %ls.%ls",
					*GetNameSafe(CharacterOwner), *GetNameSafe(UpdatedComponent));
			}
		}
	}
}

bool UCharacterMovementComponent::TryToLeaveNavWalking()
{
	SetNavWalkingPhysics(false);

	bool bSucceeded = true;
	if (CharacterOwner)
	{
		FVector CollisionFreeLocation = UpdatedComponent->GetComponentLocation();
		bSucceeded = GetWorld()->FindTeleportSpot(CharacterOwner, CollisionFreeLocation, UpdatedComponent->GetComponentRotation());
		if (bSucceeded)
		{
			CharacterOwner->SetActorLocation(CollisionFreeLocation);
		}
		else
		{
			SetNavWalkingPhysics(true);
		}
	}

	if (MovementMode == MOVE_NavWalking && bSucceeded)
	{
		SetMovementMode(DefaultLandMovementMode != MOVE_NavWalking ? DefaultLandMovementMode.GetValue() : MOVE_Walking);
	}
	else if (MovementMode != MOVE_NavWalking && !bSucceeded)
	{
		SetMovementMode(MOVE_NavWalking);
	}

	bWantsToLeaveNavWalking = !bSucceeded;
	return bSucceeded;
}

void UCharacterMovementComponent::OnTeleported()
{
	if (!HasValidData())
	{
		return;
	}

	Super::OnTeleported();

	bJustTeleported = true;

	// Find floor at current location
	UpdateFloorFromAdjustment();

	// Validate it. We don't want to pop down to walking mode from very high off the ground, but we'd like to keep walking if possible.
	const FMovementBaseInterfaceData* OldMovementBaseInterfaceData = GetMovementBaseInterfaceData();
	const bool bOldMovementBaseValid = OldMovementBaseInterfaceData && OldMovementBaseInterfaceData->IsValid();
	bool bUseNewBase = false;
	if ((bOldMovementBaseValid || CharacterMovementCVars::bRestoreFloorWhenMissingBaseComponent) && CurrentFloor.IsWalkableFloor() && CurrentFloor.FloorDist <= MAX_FLOOR_DIST && GetGravitySpaceZ(Velocity) <= 0.f)
	{
		// Close enough to land or just keep walking.
		const FMovementBaseInterfaceData NewMovementBaseInterfaceData = MovementBaseUtility::GetMovementBaseDataFromHitResult(&CurrentFloor.HitResult);
		bUseNewBase = NewMovementBaseInterfaceData.IsValid() || CharacterMovementCVars::bRestoreFloorWhenMissingBaseComponent;
	}
	else
	{
		CurrentFloor.Clear();
	}

	const bool bWasFalling = (MovementMode == MOVE_Falling);
	const bool bWasSwimming = (MovementMode == DefaultWaterMovementMode) || (MovementMode == MOVE_Swimming);

	if (CanEverSwim() && IsInWater())
	{
		if (!bWasSwimming)
		{
			SetMovementMode(DefaultWaterMovementMode);
		}
	}
	else if (!CurrentFloor.IsWalkableFloor() || (bOldMovementBaseValid && !bUseNewBase))
	{
		if (!bWasFalling && MovementMode != MOVE_Flying && MovementMode != MOVE_Custom)
		{
			SetMovementMode(MOVE_Falling);
		}
	}
	else if (bUseNewBase)
	{
		if (bWasSwimming)
		{
			SetMovementMode(DefaultLandMovementMode);
		}
		else if (bWasFalling)
		{
			ProcessLanded(CurrentFloor.HitResult, 0.f, 0);
		}
	}

	SaveBaseLocation();
}

float GetAxisDeltaRotation(float InAxisRotationRate, float DeltaTime)
{
	// Values over 360 don't do anything, see FMath::FixedTurn. However we are trying to avoid giant floats from overflowing other calculations.
	return (InAxisRotationRate >= 0.f) ? FMath::Min(InAxisRotationRate * DeltaTime, 360.f) : 360.f;
}

FRotator UCharacterMovementComponent::GetDeltaRotation(float DeltaTime) const
{
	return FRotator(GetAxisDeltaRotation(RotationRate.Pitch, DeltaTime), GetAxisDeltaRotation(RotationRate.Yaw, DeltaTime), GetAxisDeltaRotation(RotationRate.Roll, DeltaTime));
}

FRotator UCharacterMovementComponent::ComputeOrientToMovementRotation(const FRotator& CurrentRotation, float DeltaTime, FRotator& DeltaRotation) const
{
	if (Acceleration.SizeSquared() < UE_KINDA_SMALL_NUMBER)
	{
		// AI path following request can orient us in that direction (it's effectively an acceleration)
		if (bHasRequestedVelocity && RequestedVelocity.SizeSquared() > UE_KINDA_SMALL_NUMBER)
		{
			return RequestedVelocity.GetSafeNormal().Rotation();
		}

		// Don't change rotation if there is no acceleration.
		return CurrentRotation;
	}

	// Rotate toward direction of acceleration.
	return Acceleration.GetSafeNormal().Rotation();
}

bool UCharacterMovementComponent::ShouldRemainVertical() const
{
	// Always remain vertical when walking or falling.
	return IsMovingOnGround() || IsFalling();
}

void UCharacterMovementComponent::PhysicsRotation(float DeltaTime)
{
	if (!(bOrientRotationToMovement || bUseControllerDesiredRotation))
	{
		return;
	}

	if (!HasValidData() || (!CharacterOwner->GetController() && !bRunPhysicsWithNoController))
	{
		return;
	}

	FRotator CurrentRotation = UpdatedComponent->GetComponentRotation(); // Normalized
	CurrentRotation.DiagnosticCheckNaN(TEXT("CharacterMovementComponent::PhysicsRotation(): CurrentRotation"));

	FRotator DeltaRot = GetDeltaRotation(DeltaTime);
	DeltaRot.DiagnosticCheckNaN(TEXT("CharacterMovementComponent::PhysicsRotation(): GetDeltaRotation"));

	FRotator DesiredRotation = CurrentRotation;
	if (bOrientRotationToMovement)
	{
		DesiredRotation = ComputeOrientToMovementRotation(CurrentRotation, DeltaTime, DeltaRot);
	}
	else if (CharacterOwner->GetController() && bUseControllerDesiredRotation)
	{
		DesiredRotation = CharacterOwner->GetController()->GetDesiredRotation();
	}
	else if (!CharacterOwner->GetController() && bRunPhysicsWithNoController && bUseControllerDesiredRotation)
	{
		if (AController* ControllerOwner = Cast<AController>(CharacterOwner->GetOwner()))
		{
			DesiredRotation = ControllerOwner->GetDesiredRotation();
		}
	}
	else
	{
		return;
	}

	const bool bWantsToBeVertical = ShouldRemainVertical();

	if (bWantsToBeVertical)
	{
		if (HasCustomGravity())
		{
			FRotator GravityRelativeDesiredRotation = (GravityToWorldTransform * DesiredRotation.Quaternion()).Rotator();
			GravityRelativeDesiredRotation.Pitch = 0.f;
			GravityRelativeDesiredRotation.Yaw = FRotator::NormalizeAxis(GravityRelativeDesiredRotation.Yaw);
			GravityRelativeDesiredRotation.Roll = 0.f;
			DesiredRotation = (WorldToGravityTransform * GravityRelativeDesiredRotation.Quaternion()).Rotator();
		}
		else
		{
			DesiredRotation.Pitch = 0.f;
			DesiredRotation.Yaw = FRotator::NormalizeAxis(DesiredRotation.Yaw);
			DesiredRotation.Roll = 0.f;
		}
	}
	else
	{
		DesiredRotation.Normalize();
	}
	
	// Accumulate a desired new rotation.
	const float AngleTolerance = 1e-3f;

	if (!CurrentRotation.Equals(DesiredRotation, AngleTolerance))
	{
		// If we'd be prevented from becoming vertical, override the non-yaw rotation rates to allow the character to snap upright
		if (CharacterMovementCVars::bPreventNonVerticalOrientationBlock && bWantsToBeVertical)
		{
			if (FMath::IsNearlyZero(DeltaRot.Pitch))
			{
				DeltaRot.Pitch = 360.0;
			}
			if (FMath::IsNearlyZero(DeltaRot.Roll))
			{
				DeltaRot.Roll = 360.0;
			}
		}

		if (HasCustomGravity())
		{
			FRotator GravityRelativeCurrentRotation = (GravityToWorldTransform * CurrentRotation.Quaternion()).Rotator();
			FRotator GravityRelativeDesiredRotation = (GravityToWorldTransform * DesiredRotation.Quaternion()).Rotator();

			// PITCH
			if (!FMath::IsNearlyEqual(GravityRelativeCurrentRotation.Pitch, GravityRelativeDesiredRotation.Pitch, AngleTolerance))
			{
				GravityRelativeDesiredRotation.Pitch = FMath::FixedTurn(GravityRelativeCurrentRotation.Pitch, GravityRelativeDesiredRotation.Pitch, DeltaRot.Pitch);
			}

			// YAW
			if (!FMath::IsNearlyEqual(GravityRelativeCurrentRotation.Yaw, GravityRelativeDesiredRotation.Yaw, AngleTolerance))
			{
				GravityRelativeDesiredRotation.Yaw = FMath::FixedTurn(GravityRelativeCurrentRotation.Yaw, GravityRelativeDesiredRotation.Yaw, DeltaRot.Yaw);
			}

			// ROLL
			if (!FMath::IsNearlyEqual(GravityRelativeCurrentRotation.Roll, GravityRelativeDesiredRotation.Roll, AngleTolerance))
			{
				GravityRelativeDesiredRotation.Roll = FMath::FixedTurn(GravityRelativeCurrentRotation.Roll, GravityRelativeDesiredRotation.Roll, DeltaRot.Roll);
			}

			DesiredRotation = (WorldToGravityTransform * GravityRelativeDesiredRotation.Quaternion()).Rotator();
		}
		else
		{
			// PITCH
			if (!FMath::IsNearlyEqual(CurrentRotation.Pitch, DesiredRotation.Pitch, AngleTolerance))
			{
				DesiredRotation.Pitch = FMath::FixedTurn(CurrentRotation.Pitch, DesiredRotation.Pitch, DeltaRot.Pitch);
			}

			// YAW
			if (!FMath::IsNearlyEqual(CurrentRotation.Yaw, DesiredRotation.Yaw, AngleTolerance))
			{
				DesiredRotation.Yaw = FMath::FixedTurn(CurrentRotation.Yaw, DesiredRotation.Yaw, DeltaRot.Yaw);
			}

			// ROLL
			if (!FMath::IsNearlyEqual(CurrentRotation.Roll, DesiredRotation.Roll, AngleTolerance))
			{
				DesiredRotation.Roll = FMath::FixedTurn(CurrentRotation.Roll, DesiredRotation.Roll, DeltaRot.Roll);
			}
		}
		
		// Set the new rotation.
		DesiredRotation.DiagnosticCheckNaN(TEXT("CharacterMovementComponent::PhysicsRotation(): DesiredRotation"));
		MoveUpdatedComponent( FVector::ZeroVector, DesiredRotation, /*bSweep*/ false );
	}
}


void UCharacterMovementComponent::PhysicsVolumeChanged( APhysicsVolume* NewVolume )
{
	if (!HasValidData())
	{
		return;
	}
	if ( NewVolume && NewVolume->bWaterVolume )
	{
		// just entered water
		if ( !CanEverSwim() )
		{
			// AI needs to stop any current moves
			IPathFollowingAgentInterface* PFAgent = GetPathFollowingAgent();
			if (PFAgent)
			{
				//PathFollowingComp->AbortMove(*this, FPathFollowingResultFlags::MovementStop);
				PFAgent->OnUnableToMove(*this);
			}			
		}
		else if ( !IsSwimming() )
		{
			SetMovementMode(MOVE_Swimming);
		}
	}
	else if ( IsSwimming() )
	{
		// just left the water - check if should jump out
		SetMovementMode(MOVE_Falling);
		FVector JumpDir(0.f);
		FVector WallNormal(0.f);
		if ( GetGravitySpaceZ(Acceleration) > 0.f && ShouldJumpOutOfWater(JumpDir)
			&& ((JumpDir | Acceleration) > 0.f) && CheckWaterJump(JumpDir, WallNormal) ) 
		{
			JumpOutOfWater(WallNormal);
			SetGravitySpaceZ(Velocity, OutofWaterZ); //set here so physics uses this for remainder of tick
		}
	}
}


bool UCharacterMovementComponent::ShouldJumpOutOfWater(FVector& JumpDir)
{
	AController* OwnerController = CharacterOwner->GetController();
	if (OwnerController)
	{
		const FRotator ControllerRot = OwnerController->GetControlRotation();
		if ((GetGravitySpaceZ(Velocity) > 0.0f) && (ControllerRot.Pitch > JumpOutOfWaterPitch))
		{
			// if Pawn is going up and looking up, then make it jump
			JumpDir = ControllerRot.Vector();
			return true;
		}
	}
	
	return false;
}


void UCharacterMovementComponent::JumpOutOfWater(FVector WallNormal) {}


bool UCharacterMovementComponent::CheckWaterJump(FVector CheckPoint, FVector& WallNormal)
{
	if (!HasValidData())
	{
		return false;
	}
	// check if there is a wall directly in front of the swimming pawn
	CheckPoint = ProjectToGravityFloor(CheckPoint);
	FVector CheckNorm = CheckPoint.GetSafeNormal();
	float PawnCapsuleRadius, PawnCapsuleHalfHeight;
	CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(PawnCapsuleRadius, PawnCapsuleHalfHeight);
	CheckPoint = UpdatedComponent->GetComponentLocation() + 1.2f * PawnCapsuleRadius * CheckNorm;
	FHitResult HitInfo(1.f);
	FCollisionQueryParams CapsuleParams(SCENE_QUERY_STAT(CheckWaterJump), false, CharacterOwner);
	FCollisionResponseParams ResponseParam;
	InitCollisionParams(CapsuleParams, ResponseParam);
	FCollisionShape CapsuleShape = GetPawnCapsuleCollisionShape(SHRINK_None);
	const ECollisionChannel CollisionChannel = UpdatedComponent->GetCollisionObjectType();
	bool bHit = GetWorld()->SweepSingleByChannel( HitInfo, UpdatedComponent->GetComponentLocation(), CheckPoint, GetWorldToGravityTransform(), CollisionChannel, CapsuleShape, CapsuleParams, ResponseParam);
	
	if ( bHit && !HitInfo.HitObjectHandle.DoesRepresentClass(APawn::StaticClass()) )
	{
		// hit a wall - check if it is low enough
		WallNormal = -1.f * HitInfo.ImpactNormal;
		FVector Start = UpdatedComponent->GetComponentLocation();
		Start += MaxOutOfWaterStepHeight * -GetGravityDirection();
		CheckPoint = Start + 3.2f * PawnCapsuleRadius * WallNormal;
		FCollisionQueryParams LineParams(SCENE_QUERY_STAT(CheckWaterJump), true, CharacterOwner);
		FCollisionResponseParams LineResponseParam;
		InitCollisionParams(LineParams, LineResponseParam);
		bHit = GetWorld()->LineTraceSingleByChannel( HitInfo, Start, CheckPoint, CollisionChannel, LineParams, LineResponseParam );
		// if no high obstruction, or it's a valid floor, then pawn can jump out of water
		return !bHit || IsWalkable(HitInfo);
	}
	return false;
}

void UCharacterMovementComponent::AddImpulse( FVector Impulse, bool bVelocityChange )
{
	if (!Impulse.IsZero() && (MovementMode != MOVE_None) && IsActive() && HasValidData())
	{
		// handle scaling by mass
		FVector FinalImpulse = Impulse;
		if ( !bVelocityChange )
		{
			if (Mass > UE_SMALL_NUMBER)
			{
				FinalImpulse = FinalImpulse / Mass;
			}
			else
			{
				UE_LOGF(LogCharacterMovement, Warning, "Attempt to apply impulse to zero or negative Mass in CharacterMovement");
			}
		}

		PendingImpulseToApply += FinalImpulse;
	}
}

void UCharacterMovementComponent::AddForce( FVector Force )
{
	if (!Force.IsZero() && (MovementMode != MOVE_None) && IsActive() && HasValidData())
	{
		if (Mass > UE_SMALL_NUMBER)
		{
			PendingForceToApply += Force / Mass;
		}
		else
		{
			UE_LOGF(LogCharacterMovement, Warning, "Attempt to apply force to zero or negative Mass in CharacterMovement");
		}
	}
}

void UCharacterMovementComponent::MoveSmooth(const FVector& InVelocity, const float DeltaSeconds, FStepDownResult* OutStepDownResult)
{
	if (!HasValidData())
	{
		return;
	}

	// Custom movement mode.
	// Custom movement may need an update even if there is zero velocity.
	if (MovementMode == MOVE_Custom)
	{
		FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, bEnableScopedMovementUpdates);
		PhysCustom(DeltaSeconds, 0);
		return;
	}

	FVector Delta = InVelocity * DeltaSeconds;
	if (Delta.IsZero())
	{
		return;
	}

	FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, bEnableScopedMovementUpdates);

	if (IsMovingOnGround())
	{
		MoveAlongFloor(InVelocity, DeltaSeconds, OutStepDownResult);
	}
	else
	{
		FHitResult Hit(1.f);
		SafeMoveUpdatedComponent(Delta, UpdatedComponent->GetComponentQuat(), true, Hit);
	
		if (Hit.IsValidBlockingHit())
		{
			bool bSteppedUp = false;

			if (IsFlying())
			{
				if (CanStepUp(Hit))
				{
					OutStepDownResult = NULL; // No need for a floor when not walking.
					bool bShouldAttemptStepUp = false;
					bShouldAttemptStepUp = FMath::Abs(GetGravitySpaceZ(Hit.ImpactNormal)) < 0.2;
					if (bShouldAttemptStepUp)
					{
						const FVector GravDir = GetGravityDirection();
						const FVector DesiredDir = Delta.GetSafeNormal();
						const float UpDown = GravDir | DesiredDir;
						if ((UpDown < 0.5f) && (UpDown > -0.2f))
						{
							bSteppedUp = StepUp(GravDir, Delta * (1.f - Hit.Time), Hit, OutStepDownResult);
						}			
					}
				}
			}
				
			// If StepUp failed, try sliding.
			if (!bSteppedUp)
			{
				SlideAlongSurface(Delta, 1.f - Hit.Time, Hit.Normal, Hit, false);
			}
		}
	}
}


void UCharacterMovementComponent::UpdateProxyAcceleration()
{
	const FRepMovement& ReplicatedMovement = CharacterOwner->GetReplicatedMovement();
	if (ReplicatedMovement.bRepAcceleration)
	{
		Acceleration = ReplicatedMovement.Acceleration;
		AnalogInputModifier = ComputeAnalogInputModifier();
	}
	else
	{
		// If acceleration isn't replicated for simulated movement, make it non-zero for animations that may want it, based on velocity.
		// Note that this represents Acceleration with units in the range [0, 1] instead of representing cm/s^2.
		Acceleration = Velocity.GetSafeNormal();
		AnalogInputModifier = 1.0f;
	}
}

bool UCharacterMovementComponent::IsWalkable(const FHitResult& Hit) const
{
	if (!Hit.IsValidBlockingHit())
	{
		// No hit, or starting in penetration
		return false;
	}

	// Never walk up vertical surfaces.
	const FVector::FReal ImpactNormalZ = GetGravitySpaceZ(Hit.ImpactNormal);
	if (ImpactNormalZ < UE_KINDA_SMALL_NUMBER)
	{
		return false;
	}

	float TestWalkableZ = WalkableFloorZ;

	// See if this component overrides the walkable floor z.
	const UPrimitiveComponent* HitComponent = Hit.Component.Get();
	if (HitComponent)
	{
		const FWalkableSlopeOverride& SlopeOverride = HitComponent->GetWalkableSlopeOverride();
		TestWalkableZ = SlopeOverride.ModifyWalkableFloorZ(TestWalkableZ);
	}
	else
	{
		FMovementBaseInterfaceData MovementBaseData = MovementBaseUtility::GetMovementBaseDataFromHitResult(&Hit);
		if (MovementBaseData.IsValid())
		{
			const FWalkableSlopeOverride& SlopeOverride = MovementBaseData.GetBodyInstanceOwner()->GetWalkableSlopeOverride();
			TestWalkableZ = SlopeOverride.ModifyWalkableFloorZ(TestWalkableZ);
		}
	}

	// Can't walk on this surface if it is too steep.
	if (ImpactNormalZ < TestWalkableZ)
	{
		return false;
	}

	return true;
}

void UCharacterMovementComponent::SetWalkableFloorAngle(float InWalkableFloorAngle)
{
	WalkableFloorAngle = FMath::Clamp(InWalkableFloorAngle, 0.f, 90.0f);
	WalkableFloorZ = FMath::Cos(FMath::DegreesToRadians(WalkableFloorAngle));
}

void UCharacterMovementComponent::SetWalkableFloorZ(float InWalkableFloorZ)
{
	WalkableFloorZ = FMath::Clamp(InWalkableFloorZ, 0.f, 1.f);
	WalkableFloorAngle = FMath::RadiansToDegrees(FMath::Acos(WalkableFloorZ));
}

float UCharacterMovementComponent::K2_GetWalkableFloorAngle() const
{
	return GetWalkableFloorAngle();
}

float UCharacterMovementComponent::K2_GetWalkableFloorZ() const
{
	return GetWalkableFloorZ();
}



bool UCharacterMovementComponent::IsWithinEdgeTolerance(const FVector& CapsuleLocation, const FVector& TestImpactPoint, const float CapsuleRadius) const
{
	const float DistFromCenterSq = ProjectToGravityFloor(TestImpactPoint - CapsuleLocation).SizeSquared();
	const float ReducedRadiusSq = FMath::Square(FMath::Max(SWEEP_EDGE_REJECT_DISTANCE + UE_KINDA_SMALL_NUMBER, CapsuleRadius - SWEEP_EDGE_REJECT_DISTANCE));
	return DistFromCenterSq < ReducedRadiusSq;
}


void UCharacterMovementComponent::ComputeFloorDist(const FVector& CapsuleLocation, float LineDistance, float SweepDistance, FFindFloorResult& OutFloorResult, float SweepRadius, const FHitResult* DownwardSweepResult) const
{
	UE_LOGF(LogCharacterMovement, VeryVerbose, "[Role:%d] ComputeFloorDist: %ls at location %ls", (int32)CharacterOwner->GetLocalRole(), *GetNameSafe(CharacterOwner), *CapsuleLocation.ToString());
	OutFloorResult.Clear();

	float PawnRadius, PawnHalfHeight;
	CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);

	bool bSkipSweep = false;
	if (DownwardSweepResult != NULL && DownwardSweepResult->IsValidBlockingHit())
	{
		// Only if the supplied sweep was vertical and downward.
		const bool bIsDownward = GetGravitySpaceZ(DownwardSweepResult->TraceStart - DownwardSweepResult->TraceEnd) > 0;
		const bool bIsVertical = ProjectToGravityFloor(DownwardSweepResult->TraceStart - DownwardSweepResult->TraceEnd).SizeSquared() <= UE_KINDA_SMALL_NUMBER;
		if (bIsDownward && bIsVertical)
		{
			// Reject hits that are barely on the cusp of the radius of the capsule
			if (IsWithinEdgeTolerance(DownwardSweepResult->Location, DownwardSweepResult->ImpactPoint, PawnRadius))
			{
				// Don't try a redundant sweep, regardless of whether this sweep is usable.
				bSkipSweep = true;

				const bool bIsWalkable = IsWalkable(*DownwardSweepResult);
				const float FloorDist = GetGravitySpaceZ(CapsuleLocation - DownwardSweepResult->Location);
				OutFloorResult.SetFromSweep(*DownwardSweepResult, FloorDist, bIsWalkable);
				
				if (bIsWalkable)
				{
					// Use the supplied downward sweep as the floor hit result.			
					return;
				}
			}
		}
	}

	// We require the sweep distance to be >= the line distance, otherwise the HitResult can't be interpreted as the sweep result.
	if (SweepDistance < LineDistance)
	{
		ensure(SweepDistance >= LineDistance);
		return;
	}

	bool bBlockingHit = false;

	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(ComputeFloorDist), false, CharacterOwner);
	// Having a character base on a component within a cluster union will cause replication problems.
	// The issue is that ACharacter::SetBase() gets a GeometryCollectionComponent passed to it when standing on the DynamicPlatform
	// and that GC is never simulating, and since it's not simulating it's stopping the based movement flow there for simulated proxies.
	QueryParams.bTraceIntoSubComponents = true;
	QueryParams.bReplaceHitWithSubComponents = false;

	FCollisionResponseParams ResponseParam;
	InitCollisionParams(QueryParams, ResponseParam);
	const ECollisionChannel CollisionChannel = UpdatedComponent->GetCollisionObjectType();

	// Sweep test
	if (!bSkipSweep && SweepDistance > 0.f && SweepRadius > 0.f)
	{
		// Use a shorter height to avoid sweeps giving weird results if we start on a surface.
		// This also allows us to adjust out of penetrations.
		const float ShrinkScale = 0.9f;
		const float ShrinkScaleOverlap = 0.1f;
		float ShrinkHeight = (PawnHalfHeight - PawnRadius) * (1.f - ShrinkScale);
		float TraceDist = SweepDistance + ShrinkHeight;
		FCollisionShape CapsuleShape = FCollisionShape::MakeCapsule(SweepRadius, PawnHalfHeight - ShrinkHeight);

		FHitResult Hit(1.f);
		bBlockingHit = FloorSweepTest(Hit, CapsuleLocation, CapsuleLocation + TraceDist * GetGravityDirection(), CollisionChannel, CapsuleShape, QueryParams, ResponseParam);

		if (bBlockingHit)
		{
			// Reject hits adjacent to us, we only care about hits on the bottom portion of our capsule.
			// Check 2D distance to impact point, reject if within a tolerance from radius.
			if (Hit.bStartPenetrating || !IsWithinEdgeTolerance(CapsuleLocation, Hit.ImpactPoint, CapsuleShape.Capsule.Radius))
			{
				// Use a capsule with a slightly smaller radius and shorter height to avoid the adjacent object.
				// Capsule must not be nearly zero or the trace will fall back to a line trace from the start point and have the wrong length.
				CapsuleShape.Capsule.Radius = FMath::Max(0.f, CapsuleShape.Capsule.Radius - SWEEP_EDGE_REJECT_DISTANCE - UE_KINDA_SMALL_NUMBER);
				if (!CapsuleShape.IsNearlyZero())
				{
					ShrinkHeight = (PawnHalfHeight - PawnRadius) * (1.f - ShrinkScaleOverlap);
					TraceDist = SweepDistance + ShrinkHeight;
					CapsuleShape.Capsule.HalfHeight = FMath::Max(PawnHalfHeight - ShrinkHeight, CapsuleShape.Capsule.Radius);
					Hit.Reset(1.f, false);

					bBlockingHit = FloorSweepTest(Hit, CapsuleLocation, CapsuleLocation + TraceDist * GetGravityDirection(), CollisionChannel, CapsuleShape, QueryParams, ResponseParam);
				}
			}

			// Reduce hit distance by ShrinkHeight because we shrank the capsule for the trace.
			// We allow negative distances here, because this allows us to pull out of penetrations.
			const float MaxPenetrationAdjust = FMath::Max(MAX_FLOOR_DIST, PawnRadius);
			const float SweepResult = FMath::Max(-MaxPenetrationAdjust, Hit.Time * TraceDist - ShrinkHeight);

			OutFloorResult.SetFromSweep(Hit, SweepResult, false);
			if (Hit.IsValidBlockingHit() && IsWalkable(Hit))
			{		
				if (SweepResult <= SweepDistance)
				{
					// Hit within test distance.
					OutFloorResult.bWalkableFloor = true;
					return;
				}
			}
		}
	}

	// Since we require a longer sweep than line trace, we don't want to run the line trace if the sweep missed everything.
	// We do however want to try a line trace if the sweep was stuck in penetration.
	if (!OutFloorResult.bBlockingHit && !OutFloorResult.HitResult.bStartPenetrating)
	{
		OutFloorResult.FloorDist = SweepDistance;
		return;
	}

	// Line trace
	if (LineDistance > 0.f)
	{
		const float ShrinkHeight = PawnHalfHeight;
		const FVector LineTraceStart = CapsuleLocation;	
		const float TraceDist = LineDistance + ShrinkHeight;
		const FVector Down = TraceDist * GetGravityDirection(); 
		QueryParams.TraceTag = SCENE_QUERY_STAT_NAME_ONLY(FloorLineTrace);

		FHitResult Hit(1.f);
		bBlockingHit = GetWorld()->LineTraceSingleByChannel(Hit, LineTraceStart, LineTraceStart + Down, CollisionChannel, QueryParams, ResponseParam);

		if (bBlockingHit)
		{
			if (Hit.Time > 0.f)
			{
				// Reduce hit distance by ShrinkHeight because we started the trace higher than the base.
				// We allow negative distances here, because this allows us to pull out of penetrations.
				const float MaxPenetrationAdjust = FMath::Max(MAX_FLOOR_DIST, PawnRadius);
				const float LineResult = FMath::Max(-MaxPenetrationAdjust, Hit.Time * TraceDist - ShrinkHeight);

				OutFloorResult.bBlockingHit = true;
				if (LineResult <= LineDistance && IsWalkable(Hit))
				{
					OutFloorResult.SetFromLineTrace(Hit, OutFloorResult.FloorDist, LineResult, true);
					return;
				}
			}
		}
	}
	
	// No hits were acceptable.
	OutFloorResult.bWalkableFloor = false;
}


void UCharacterMovementComponent::FindFloor(const FVector& CapsuleLocation, FFindFloorResult& OutFloorResult, bool bCanUseCachedLocation, const FHitResult* DownwardSweepResult) const
{
	SCOPE_CYCLE_COUNTER(STAT_CharFindFloor);

	// No collision, no floor...
	if (!HasValidData() || !UpdatedComponent->IsQueryCollisionEnabled())
	{
		OutFloorResult.Clear();
		return;
	}

	UE_LOGF(LogCharacterMovement, VeryVerbose, "[Role:%d] FindFloor: %ls at location %ls", (int32)CharacterOwner->GetLocalRole(), *GetNameSafe(CharacterOwner), *CapsuleLocation.ToString());
	check(CharacterOwner->GetCapsuleComponent());

	// Increase height check slightly if walking, to prevent floor height adjustment from later invalidating the floor result.
	const float HeightCheckAdjust = (IsMovingOnGround() ? MAX_FLOOR_DIST + UE_KINDA_SMALL_NUMBER : -MAX_FLOOR_DIST);

	float FloorSweepTraceDist = FMath::Max(MAX_FLOOR_DIST, MaxStepHeight + HeightCheckAdjust);
	float FloorLineTraceDist = FloorSweepTraceDist;
	bool bNeedToValidateFloor = true;
	
	// Sweep floor
	if (FloorLineTraceDist > 0.f || FloorSweepTraceDist > 0.f)
	{
		UCharacterMovementComponent* MutableThis = const_cast<UCharacterMovementComponent*>(this);

		if ( bAlwaysCheckFloor || !bCanUseCachedLocation || bForceNextFloorCheck || bJustTeleported )
		{
			MutableThis->bForceNextFloorCheck = false;
			ComputeFloorDist(CapsuleLocation, FloorLineTraceDist, FloorSweepTraceDist, OutFloorResult, CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleRadius(), DownwardSweepResult);
		}
		else
		{
			// Force floor check if base has collision disabled or if it does not block us.
			const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
			
			const bool bHasValidMovementBase = MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid();
			const AActor* BaseActor = bHasValidMovementBase ? Cast<const AActor>(MovementBaseInterfaceData->GetMovementBaseObject()) : nullptr;
			const ECollisionChannel CollisionChannel = UpdatedComponent->GetCollisionObjectType();

			if (bHasValidMovementBase)
			{
				MutableThis->bForceNextFloorCheck = !CollisionEnabledHasQuery(MovementBaseInterfaceData->GetBodyInstanceOwner()->GetCollisionEnabled())
				|| MovementBaseInterfaceData->GetBodyInstanceOwner()->GetCollisionResponseToChannel(CollisionChannel) != ECR_Block
				|| MovementBaseUtility::IsDynamicBase(MovementBaseInterfaceData);
			}

			const bool IsActorBasePendingKill = BaseActor && !IsValid(BaseActor);

			if ( !bForceNextFloorCheck && !IsActorBasePendingKill && bHasValidMovementBase)
			{
				//UE_LOGF(LogCharacterMovement, Log, "%ls SKIP check for floor", *CharacterOwner->GetName());
				OutFloorResult = CurrentFloor;
				bNeedToValidateFloor = false;
			}
			else
			{
				MutableThis->bForceNextFloorCheck = false;
				ComputeFloorDist(CapsuleLocation, FloorLineTraceDist, FloorSweepTraceDist, OutFloorResult, CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleRadius(), DownwardSweepResult);
			}
		}
	}

	// OutFloorResult.HitResult is now the result of the vertical floor check.
	// See if we should try to "perch" at this location.
	if (bNeedToValidateFloor && OutFloorResult.bBlockingHit && !OutFloorResult.bLineTrace)
	{
		const bool bCheckRadius = true;
		if (ShouldComputePerchResult(OutFloorResult.HitResult, bCheckRadius))
		{
			float MaxPerchFloorDist = FMath::Max(MAX_FLOOR_DIST, MaxStepHeight + HeightCheckAdjust);
			if (IsMovingOnGround())
			{
				MaxPerchFloorDist += FMath::Max(0.f, PerchAdditionalHeight);
			}

			FFindFloorResult PerchFloorResult;
			if (ComputePerchResult(GetValidPerchRadius(), OutFloorResult.HitResult, MaxPerchFloorDist, PerchFloorResult))
			{
				// Don't allow the floor distance adjustment to push us up too high, or we will move beyond the perch distance and fall next time.
				const float AvgFloorDist = (MIN_FLOOR_DIST + MAX_FLOOR_DIST) * 0.5f;
				const float MoveUpDist = (AvgFloorDist - OutFloorResult.FloorDist);
				if (MoveUpDist + PerchFloorResult.FloorDist >= MaxPerchFloorDist)
				{
					OutFloorResult.FloorDist = AvgFloorDist;
				}

				// If the regular capsule is on an unwalkable surface but the perched one would allow us to stand, override the normal to be one that is walkable.
				if (!OutFloorResult.bWalkableFloor)
				{
					// Floor distances are used as the distance of the regular capsule to the point of collision, to make sure AdjustFloorHeight() behaves correctly.
					OutFloorResult.SetFromLineTrace(PerchFloorResult.HitResult, OutFloorResult.FloorDist, FMath::Max(OutFloorResult.FloorDist, MIN_FLOOR_DIST), true);
				}
			}
			else
			{
				// We had no floor (or an invalid one because it was unwalkable), and couldn't perch here, so invalidate floor (which will cause us to start falling).
				OutFloorResult.bWalkableFloor = false;
			}
		}
	}
}


void UCharacterMovementComponent::K2_FindFloor(FVector CapsuleLocation, FFindFloorResult& FloorResult) const
{
	const bool SavedForceNextFloorCheck(bForceNextFloorCheck);
	FindFloor(CapsuleLocation, FloorResult, false);
	
	// FindFloor clears this, but this is only a test not done during normal movement.
	UCharacterMovementComponent* MutableThis = const_cast<UCharacterMovementComponent*>(this);
	MutableThis->bForceNextFloorCheck = SavedForceNextFloorCheck;
}

void UCharacterMovementComponent::K2_ComputeFloorDist(FVector CapsuleLocation, float LineDistance, float SweepDistance, float SweepRadius, FFindFloorResult& FloorResult) const
{
	if (HasValidData())
	{
		SweepDistance = FMath::Max(SweepDistance, 0.f);
		LineDistance = FMath::Clamp(LineDistance, 0.f, SweepDistance);
		SweepRadius = FMath::Max(SweepRadius, 0.f);

		ComputeFloorDist(CapsuleLocation, LineDistance, SweepDistance, FloorResult, SweepRadius);
	}
}


bool UCharacterMovementComponent::FloorSweepTest(
	FHitResult& OutHit,
	const FVector& Start,
	const FVector& End,
	ECollisionChannel TraceChannel,
	const struct FCollisionShape& CollisionShape,
	const struct FCollisionQueryParams& Params,
	const struct FCollisionResponseParams& ResponseParam
	) const
{
	bool bBlockingHit = false;

	if (!bUseFlatBaseForFloorChecks)
	{
		bBlockingHit = GetWorld()->SweepSingleByChannel(OutHit, Start, End, GetWorldToGravityTransform(), TraceChannel, CollisionShape, Params, ResponseParam);
	}
	else
	{
		// Test with a box that is enclosed by the capsule.
		const float CapsuleRadius = CollisionShape.GetCapsuleRadius();
		const float CapsuleHeight = CollisionShape.GetCapsuleHalfHeight();
		const FCollisionShape BoxShape = FCollisionShape::MakeBox(FVector(CapsuleRadius * 0.707f, CapsuleRadius * 0.707f, CapsuleHeight));

		// First test with the box rotated so the corners are along the major axes (ie rotated 45 degrees).
		bBlockingHit = GetWorld()->SweepSingleByChannel(OutHit, Start, End, FQuat(GetGravityDirection(), UE_PI * 0.25f), TraceChannel, BoxShape, Params, ResponseParam);

		if (!bBlockingHit)
		{
			// Test again with the same box, not rotated.
			OutHit.Reset(1.f, false);
			bBlockingHit = GetWorld()->SweepSingleByChannel(OutHit, Start, End, GetWorldToGravityTransform(), TraceChannel, BoxShape, Params, ResponseParam);
		}
	}

	return bBlockingHit;
}


bool UCharacterMovementComponent::IsValidLandingSpot(const FVector& CapsuleLocation, const FHitResult& Hit) const
{
	if (!Hit.bBlockingHit)
	{
		return false;
	}

	// Skip some checks if penetrating. Penetration will be handled by the FindFloor call (using a smaller capsule)
	if (!Hit.bStartPenetrating)
	{
		// Reject unwalkable floor normals.
		if (!IsWalkable(Hit))
		{
			return false;
		}

		float PawnRadius, PawnHalfHeight;
		CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);

		// Reject hits that are above our lower hemisphere (can happen when sliding down a vertical surface).
		const FVector::FReal LowerHemisphereZ = GetGravitySpaceZ(Hit.Location) - PawnHalfHeight + PawnRadius;
		if (GetGravitySpaceZ(Hit.ImpactPoint) >= LowerHemisphereZ)
		{
			return false;
		}

		// Reject hits that are barely on the cusp of the radius of the capsule
		if (!IsWithinEdgeTolerance(Hit.Location, Hit.ImpactPoint, PawnRadius))
		{
			return false;
		}
	}
	else
	{
		// Penetrating
		if (GetGravitySpaceZ(Hit.Normal) < UE_KINDA_SMALL_NUMBER)
		{
			// Normal is nearly horizontal or downward, that's a penetration adjustment next to a vertical or overhanging wall. Don't pop to the floor.
			return false;
		}
	}

	FFindFloorResult FloorResult;
	FindFloor(CapsuleLocation, FloorResult, false, &Hit);

	if (!FloorResult.IsWalkableFloor())
	{
		return false;
	}

	return true;
}


bool UCharacterMovementComponent::ShouldCheckForValidLandingSpot(float DeltaTime, const FVector& Delta, const FHitResult& Hit) const
{
	// See if we hit an edge of a surface on the lower portion of the capsule.
	// In this case the normal will not equal the impact normal, and a downward sweep may find a walkable surface on top of the edge.
	if (GetGravitySpaceZ(Hit.Normal) > UE_KINDA_SMALL_NUMBER && !Hit.Normal.Equals(Hit.ImpactNormal))
	{
		const FVector PawnLocation = UpdatedComponent->GetComponentLocation();
		if (IsWithinEdgeTolerance(PawnLocation, Hit.ImpactPoint, CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleRadius()))
		{						
			return true;
		}
	}

	return false;
}


float UCharacterMovementComponent::GetPerchRadiusThreshold() const
{
	// Don't allow negative values.
	return FMath::Max(0.f, PerchRadiusThreshold);
}


float UCharacterMovementComponent::GetValidPerchRadius() const
{
	if (CharacterOwner)
	{
		const float PawnRadius = CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleRadius();
		return FMath::Clamp(PawnRadius - GetPerchRadiusThreshold(), 0.11f, PawnRadius);
	}
	return 0.f;
}


bool UCharacterMovementComponent::ShouldComputePerchResult(const FHitResult& InHit, bool bCheckRadius) const
{
	if (!InHit.IsValidBlockingHit())
	{
		return false;
	}

	// Don't try to perch if the edge radius is very small.
	if (GetPerchRadiusThreshold() <= SWEEP_EDGE_REJECT_DISTANCE)
	{
		return false;
	}

	if (bCheckRadius)
	{
		const float DistFromCenterSq = ProjectToGravityFloor(InHit.ImpactPoint - InHit.Location).SizeSquared();
		const float StandOnEdgeRadius = GetValidPerchRadius();
		if (DistFromCenterSq <= FMath::Square(StandOnEdgeRadius))
		{
			// Already within perch radius.
			return false;
		}
	}
	
	return true;
}


bool UCharacterMovementComponent::ComputePerchResult(const float TestRadius, const FHitResult& InHit, const float InMaxFloorDist, FFindFloorResult& OutPerchFloorResult) const
{
	if (InMaxFloorDist <= 0.f)
	{
		return false;
	}

	// Sweep further than actual requested distance, because a reduced capsule radius means we could miss some hits that the normal radius would contact.
	float PawnRadius, PawnHalfHeight;
	CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);
	const FVector CapsuleLocation = (bUseFlatBaseForFloorChecks ? InHit.TraceStart : InHit.Location);

	const float InHitAboveBase = FMath::Max<float>(0.f, GetGravitySpaceZ(InHit.ImpactPoint - CapsuleLocation) + PawnHalfHeight);
	const float PerchLineDist = FMath::Max(0.f, InMaxFloorDist - InHitAboveBase);
	const float PerchSweepDist = FMath::Max(0.f, InMaxFloorDist);

	const float ActualSweepDist = PerchSweepDist + PawnRadius;
	ComputeFloorDist(CapsuleLocation, PerchLineDist, ActualSweepDist, OutPerchFloorResult, TestRadius);

	if (!OutPerchFloorResult.IsWalkableFloor())
	{
		return false;
	}
	else if (InHitAboveBase + OutPerchFloorResult.FloorDist > InMaxFloorDist)
	{
		// Hit something past max distance
		OutPerchFloorResult.bWalkableFloor = false;
		return false;
	}

	return true;
}


bool UCharacterMovementComponent::CanStepUp(const FHitResult& Hit) const
{
	if (!Hit.IsValidBlockingHit() || !HasValidData() || MovementMode == MOVE_Falling)
	{
		return false;
	}

	// No component for "fake" hits when we are on a known good base.
	const UPrimitiveComponent* HitComponent = Hit.Component.Get();
	if (!HitComponent)
	{
		return true;
	}

	if (!HitComponent->CanCharacterStepUp(CharacterOwner))
	{
		return false;
	}

	// No actor for "fake" hits when we are on a known good base.
	
	if (!Hit.HitObjectHandle.IsValid())
	{
		 return true;
	}

	const AActor* HitActor = Hit.HitObjectHandle.GetManagingActor();
	if (!HitActor->CanBeBaseForCharacter(CharacterOwner))
	{
		return false;
	}

	return true;
}


bool UCharacterMovementComponent::StepUp(const FVector& GravDir, const FVector& Delta, const FHitResult &InHit, FStepDownResult* OutStepDownResult)
{
	SCOPE_CYCLE_COUNTER(STAT_CharStepUp);

	if (!CanStepUp(InHit) || MaxStepHeight <= 0.f)
	{
		return false;
	}

	const FVector OldLocation = UpdatedComponent->GetComponentLocation();
	float PawnRadius, PawnHalfHeight;
	CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);

	// Don't bother stepping up if top of capsule is hitting something.
	const float InitialImpactZ = InHit.ImpactPoint | -GravDir;
	const float OldLocationZ = OldLocation | -GravDir;
	if (InitialImpactZ > OldLocationZ + (PawnHalfHeight - PawnRadius))
	{
		return false;
	}

	if (GravDir.IsZero())
	{
		return false;
	}

	// Gravity should be a normalized direction
	ensure(GravDir.IsNormalized());

	float StepTravelUpHeight = MaxStepHeight;
	float StepTravelDownHeight = StepTravelUpHeight;
	const float StepSideZ = -1.f * FVector::DotProduct(InHit.ImpactNormal, GravDir);
	float PawnInitialFloorBaseZ = OldLocationZ - PawnHalfHeight;
	float PawnFloorPointZ = PawnInitialFloorBaseZ;

	if (IsMovingOnGround() && CurrentFloor.IsWalkableFloor())
	{
		// Since we float a variable amount off the floor, we need to enforce max step height off the actual point of impact with the floor.
		const float FloorDist = FMath::Max(0.f, CurrentFloor.GetDistanceToFloor());
		PawnInitialFloorBaseZ -= FloorDist;
		StepTravelUpHeight = FMath::Max(StepTravelUpHeight - FloorDist, 0.f);
		StepTravelDownHeight = (MaxStepHeight + MAX_FLOOR_DIST*2.f);

		const bool bHitVerticalFace = !IsWithinEdgeTolerance(InHit.Location, InHit.ImpactPoint, PawnRadius);
		if (!CurrentFloor.bLineTrace && !bHitVerticalFace)
		{
			PawnFloorPointZ = CurrentFloor.HitResult.ImpactPoint | -GravDir;
		}
		else
		{
			// Base floor point is the base of the capsule moved down by how far we are hovering over the surface we are hitting.
			PawnFloorPointZ -= CurrentFloor.FloorDist;
		}
	}

	// Don't step up if the impact is below us, accounting for distance from floor.
	if (InitialImpactZ <= PawnInitialFloorBaseZ)
	{
		return false;
	}

	// Scope our movement updates, and do not apply them until all intermediate moves are completed.
	FScopedCapsuleMovementUpdate ScopedStepUpMovement(UpdatedComponent, true);

	// step up - treat as vertical wall
	FHitResult SweepUpHit(1.f);
	const FQuat PawnRotation = UpdatedComponent->GetComponentQuat();
	MoveUpdatedComponent(-GravDir * StepTravelUpHeight, PawnRotation, true, &SweepUpHit);

	if (SweepUpHit.bStartPenetrating)
	{
		// Undo movement
		ScopedStepUpMovement.RevertMove();
		return false;
	}

	// step fwd
	FHitResult Hit(1.f);
	MoveUpdatedComponent( Delta, PawnRotation, true, &Hit);

	// Check result of forward movement
	if (Hit.bBlockingHit)
	{
		if (Hit.bStartPenetrating)
		{
			// Undo movement
			ScopedStepUpMovement.RevertMove();
			return false;
		}

		// If we hit something above us and also something ahead of us, we should notify about the upward hit as well.
		// The forward hit will be handled later (in the bSteppedOver case below).
		// In the case of hitting something above but not forward, we are not blocked from moving so we don't need the notification.
		if (SweepUpHit.bBlockingHit && Hit.bBlockingHit)
		{
			HandleImpact(SweepUpHit);
		}

		// pawn ran into a wall
		HandleImpact(Hit);
		if (IsFalling())
		{
			return true;
		}

		// adjust and try again
		const float ForwardHitTime = Hit.Time;
		const float ForwardSlideAmount = SlideAlongSurface(Delta, 1.f - Hit.Time, Hit.Normal, Hit, true);
		
		if (IsFalling())
		{
			ScopedStepUpMovement.RevertMove();
			return false;
		}

		// If both the forward hit and the deflection got us nowhere, there is no point in this step up.
		if (ForwardHitTime == 0.f && ForwardSlideAmount == 0.f)
		{
			ScopedStepUpMovement.RevertMove();
			return false;
		}
	}
	
	// Step down
	MoveUpdatedComponent(GravDir * StepTravelDownHeight, UpdatedComponent->GetComponentQuat(), true, &Hit);

	// If step down was initially penetrating abort the step up
	if (Hit.bStartPenetrating)
	{
		ScopedStepUpMovement.RevertMove();
		return false;
	}

	FStepDownResult StepDownResult;
	if (Hit.IsValidBlockingHit())
	{	
		// See if this step sequence would have allowed us to travel higher than our max step height allows.
		const float DeltaZ = (Hit.ImpactPoint | -GravDir) - PawnFloorPointZ;
		if (DeltaZ > MaxStepHeight)
		{
			//UE_LOGF(LogCharacterMovement, VeryVerbose, "- Reject StepUp (too high Height %.3f) up from floor base %f to %f", DeltaZ, PawnInitialFloorBaseZ, NewLocation.Z);
			ScopedStepUpMovement.RevertMove();
			return false;
		}

		// Reject unwalkable surface normals here.
		if (!IsWalkable(Hit))
		{
			// Reject if normal opposes movement direction
			const bool bNormalTowardsMe = (Delta | Hit.ImpactNormal) < 0.f;
			if (bNormalTowardsMe)
			{
				//UE_LOGF(LogCharacterMovement, VeryVerbose, "- Reject StepUp (unwalkable normal %ls opposed to movement)", *Hit.ImpactNormal.ToString());
				ScopedStepUpMovement.RevertMove();
				return false;
			}

			// Also reject if we would end up being higher than our starting location by stepping down.
			// It's fine to step down onto an unwalkable normal below us, we will just slide off. Rejecting those moves would prevent us from being able to walk off the edge.
			if ((Hit.Location | -GravDir) > OldLocationZ)
			{
				//UE_LOGF(LogCharacterMovement, VeryVerbose, "- Reject StepUp (unwalkable normal %ls above old position)", *Hit.ImpactNormal.ToString());
				ScopedStepUpMovement.RevertMove();
				return false;
			}
		}

		// Reject moves where the downward sweep hit something very close to the edge of the capsule. This maintains consistency with FindFloor as well.
		if (!IsWithinEdgeTolerance(Hit.Location, Hit.ImpactPoint, PawnRadius))
		{
			//UE_LOGF(LogCharacterMovement, VeryVerbose, "- Reject StepUp (outside edge tolerance)");
			ScopedStepUpMovement.RevertMove();
			return false;
		}

		// Don't step up onto invalid surfaces if traveling higher.
		if (DeltaZ > 0.f && !CanStepUp(Hit))
		{
			//UE_LOGF(LogCharacterMovement, VeryVerbose, "- Reject StepUp (up onto surface with !CanStepUp())");
			ScopedStepUpMovement.RevertMove();
			return false;
		}

		// See if we can validate the floor as a result of this step down. In almost all cases this should succeed, and we can avoid computing the floor outside this method.
		if (OutStepDownResult != NULL)
		{
			FindFloor(UpdatedComponent->GetComponentLocation(), StepDownResult.FloorResult, false, &Hit);

			// Reject unwalkable normals if we end up higher than our initial height.
			// It's fine to walk down onto an unwalkable surface, don't reject those moves.
			if ((Hit.Location | -GravDir) > OldLocationZ)
			{
				// We should reject the floor result if we are trying to step up an actual step where we are not able to perch (this is rare).
				// In those cases we should instead abort the step up and try to slide along the stair.
				if (!StepDownResult.FloorResult.bBlockingHit && StepSideZ < CharacterMovementConstants::MAX_STEP_SIDE_Z)
				{
					ScopedStepUpMovement.RevertMove();
					return false;
				}
			}

			StepDownResult.bComputedFloor = true;
		}
	}
	
	// Copy step down result.
	if (OutStepDownResult != NULL)
	{
		*OutStepDownResult = StepDownResult;
	}

	// Don't recalculate velocity based on this height adjustment, if considering vertical adjustments.
	bJustTeleported |= !bMaintainHorizontalGroundVelocity;

	return true;
}

void UCharacterMovementComponent::SetBase(UPrimitiveComponent* NewBase, const FName BoneName, bool bNotifyActor)
{
	FMovementBaseInterfaceData MovementBaseInterfaceData(NewBase);
	SetBase(&MovementBaseInterfaceData, BoneName, bNotifyActor);
}

void UCharacterMovementComponent::HandleImpact(const FHitResult& Impact, float TimeSlice, const FVector& MoveDelta)
{
	SCOPE_CYCLE_COUNTER(STAT_CharHandleImpact);

	if (CharacterOwner)
	{
		CharacterOwner->MoveBlockedBy(Impact);
	}

	IPathFollowingAgentInterface* PFAgent = GetPathFollowingAgent();
	if (PFAgent)
	{
		// Also notify path following!
		PFAgent->OnMoveBlockedBy(Impact);
	}

	if (Impact.HitObjectHandle.DoesRepresentClass(APawn::StaticClass()))
	{
		APawn* OtherPawn = Impact.HitObjectHandle.FetchActor<APawn>();
		NotifyBumpedPawn(OtherPawn);
	}

	if (bEnablePhysicsInteraction)
	{
		const FVector FallingAcceleration = -GetGravityDirection() * GetGravityZ();
		const FVector ForceAccel = Acceleration + (IsFalling() ? FallingAcceleration : FVector::ZeroVector);
		ApplyImpactPhysicsForces(Impact, ForceAccel, Velocity);
	}
}

void UCharacterMovementComponent::ApplyImpactPhysicsForces(const FHitResult& Impact, const FVector& ImpactAcceleration, const FVector& ImpactVelocity)
{
	if (bEnablePhysicsInteraction && Impact.bBlockingHit )
	{
		if (UPrimitiveComponent* ImpactComponent = Impact.GetComponent())
		{
			FVector ForcePoint = Impact.ImpactPoint;
			float BodyMass = 1.0f; // set to 1 as this is used as a multiplier

			bool bCanBePushed = false;
			FBodyInstance* BI = ImpactComponent->GetBodyInstance(Impact.BoneName);
			if(BI != nullptr && BI->IsInstanceSimulatingPhysics())
			{
				BodyMass = FMath::Max(BI->GetBodyMass(), 1.0f);

				if(bPushForceUsingZOffset)
				{
					FBox Bounds = BI->GetBodyBounds();

					FVector Center, Extents;
					Bounds.GetCenterAndExtents(Center, Extents);

					if (!Extents.IsNearlyZero())
					{
						SetGravitySpaceZ(ForcePoint, GetGravitySpaceZ(Center) + GetGravitySpaceZ(Extents) * PushForcePointZOffsetFactor);
					}
				}

				bCanBePushed = true;
			}
			else if (CharacterMovementCVars::bGeometryCollectionImpulseWorkAround)
			{
				const FName ClassName = ImpactComponent->GetClass()->GetFName();
				const FName GeometryCollectionClassName("GeometryCollectionComponent");
				if (ClassName == GeometryCollectionClassName && ImpactComponent->BodyInstance.bSimulatePhysics)
				{
					// in some case GetBodyInstance can return null while the BodyInstance still exists ( geometry collection component for example )
					// but we cannot check for its component directly here because of modules cyclic dependencies
					// todo(chaos): change this logic to be more driven at the primitive component level to avoid the high level classes to have to be aware of the different component

					// because of the above limititation we have to ignore bPushForceUsingZOffset

					bCanBePushed = true;
				}
			}

			if (bCanBePushed)
			{
				FVector Force = Impact.ImpactNormal * -1.0f;

				float PushForceModificator = 1.0f;

				const FVector ComponentVelocity = ImpactComponent->GetPhysicsLinearVelocity(Impact.BoneName);
				const FVector VirtualVelocity = ImpactAcceleration.IsZero() ? ImpactVelocity : ImpactAcceleration.GetSafeNormal() * GetMaxSpeed();

				float Dot = 0.0f;

				if (bScalePushForceToVelocity && !ComponentVelocity.IsNearlyZero())
				{
					Dot = ComponentVelocity | VirtualVelocity;

					if (Dot > 0.0f && Dot < 1.0f)
					{
						PushForceModificator *= Dot;
					}
				}

				if (bPushForceScaledToMass)
				{
					PushForceModificator *= BodyMass;
				}
				const bool bClientAuthScheduledPushForces = CharacterMovementCVars::bEnableClientAuthScheduledPushForces && Chaos::FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled();
	
#if UE_WITH_REMOTE_OBJECT_HANDLE
				PushForceModificator *= PhysicsForceSubsteppingFactor;
#endif

				Force *= PushForceModificator;
				const float ZeroVelocityTolerance = 1.0f;
				if (ComponentVelocity.IsNearlyZero(ZeroVelocityTolerance))
				{
					Force *= InitialPushForceFactor;

					if (CharacterMovementCVars::bApplyCharacterCollisionForceAsImpulse)
					{
						if (!bClientAuthScheduledPushForces)
						{
							ImpactComponent->AddImpulseAtLocation(Force, ForcePoint, Impact.BoneName);
						}
						else
						{
							ApplyAsyncPhysicsStateAction(ImpactComponent, Impact.BoneName, EPhysicsStateAction::AddImpulseAtPosition, Force, ForcePoint);
						}
					}
					else
					{
						//Originally the force is applied as impulse, which was inaccurate. The extra scaling is applied so that the force is calibrated to match the old behavior at 30fps. 
						const float ExtraForceFactor = 30.f;
						Force *= ExtraForceFactor;

						if (!bClientAuthScheduledPushForces)
						{
							ImpactComponent->AddForceAtLocation(Force, ForcePoint, Impact.BoneName);
						}
						else
						{
							ApplyAsyncPhysicsStateAction(ImpactComponent, Impact.BoneName, EPhysicsStateAction::AddForceAtPosition, Force, ForcePoint);
						}
					}
				}
				else
				{
					Force *= PushForceFactor;
					if (!bClientAuthScheduledPushForces)
					{
						ImpactComponent->AddForceAtLocation(Force, ForcePoint, Impact.BoneName);
					}
					else
					{
						ApplyAsyncPhysicsStateAction(ImpactComponent, Impact.BoneName, EPhysicsStateAction::AddForceAtPosition, Force, ForcePoint);
					}
				}
			}
		}
	}
}

FString UCharacterMovementComponent::GetMovementName() const
{
	if( CharacterOwner )
	{
		if ( CharacterOwner->GetRootComponent() && CharacterOwner->GetRootComponent()->IsSimulatingPhysics() )
		{
			return TEXT("Rigid Body");
		}
	}

	// Using character movement
	switch( MovementMode )
	{
		case MOVE_None:				return TEXT("NULL");
		case MOVE_Walking:			return TEXT("Walking");
		case MOVE_NavWalking:		return TEXT("NavWalking");
		case MOVE_Falling:			return TEXT("Falling");
		case MOVE_Swimming:			return TEXT("Swimming");
		case MOVE_Flying:			return TEXT("Flying");
		case MOVE_Custom:			return TEXT("Custom");
		default:					break;
	}
	return TEXT("Unknown");
}

void UCharacterMovementComponent::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	if ( CharacterOwner == NULL )
	{
		return;
	}

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	DisplayDebugManager.SetDrawColor(FColor::White);
	FString T = FString::Printf(TEXT("CHARACTER MOVEMENT Floor %s Crouched %i"), *CurrentFloor.HitResult.ImpactNormal.ToString(), IsCrouching());
	DisplayDebugManager.DrawString(T);

	T = FString::Printf(TEXT("Updated Component: %s"), *UpdatedComponent->GetName());
	DisplayDebugManager.DrawString(T);

	T = FString::Printf(TEXT("Acceleration: %s"), *Acceleration.ToCompactString());
	DisplayDebugManager.DrawString(T);

	T = FString::Printf(TEXT("bForceMaxAccel: %i"), bForceMaxAccel);
	DisplayDebugManager.DrawString(T);

	T = FString::Printf(TEXT("RootMotionSources: %d active"), CurrentRootMotion.RootMotionSources.Num());
	DisplayDebugManager.DrawString(T);

	APhysicsVolume * PhysicsVolume = GetPhysicsVolume();

	const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
	const UObject* BodyInstanceSourceObject = MovementBaseInterfaceData && MovementBaseInterfaceData->GetBodyInstanceOwner() ? MovementBaseInterfaceData->GetBodyInstanceOwner()->GetSourceObject() : nullptr;
	const UObject* BodyInstanceSourceObjectOwner = MovementBaseInterfaceData && MovementBaseInterfaceData->GetBodyInstanceOwner() ? MovementBaseInterfaceData->GetBodyInstanceOwner()->GetSourceObjectOwner() : nullptr;

	T = FString::Printf(TEXT("%s In physicsvolume %s on base %s component %s gravity %f"), *GetMovementName(), (PhysicsVolume ? *PhysicsVolume->GetName() : TEXT("None")),
		(BodyInstanceSourceObjectOwner ? *BodyInstanceSourceObjectOwner->GetName() : TEXT("None")), (BodyInstanceSourceObject ? *BodyInstanceSourceObject->GetName() : TEXT("None")), GetGravityZ());
	DisplayDebugManager.DrawString(T);
}

float UCharacterMovementComponent::VisualizeMovement() const
{
	float HeightOffset = 0.f;
	const float OffsetPerElement = 10.0f;
	if (CharacterOwner == nullptr)
	{
		return HeightOffset;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const FVector TopOfCapsule = GetActorLocation() + CharacterOwner->GetSimpleCollisionHalfHeight() * -GetGravityDirection();
	
	// Position
	{
		const FColor DebugColor = FColor::White;
		const FVector DebugLocation = TopOfCapsule + HeightOffset * -GetGravityDirection();
		FString DebugText = FString::Printf(TEXT("Position: %s"), *GetActorLocation().ToCompactString());
		DrawDebugString(GetWorld(), DebugLocation, DebugText, nullptr, DebugColor, 0.f, true);
	}

	// Velocity
	{
		const FColor DebugColor = FColor::Green;
		HeightOffset += OffsetPerElement;
		const FVector DebugLocation = TopOfCapsule + HeightOffset * -GetGravityDirection();
		DrawDebugDirectionalArrow(GetWorld(), DebugLocation - 5.f * -GetGravityDirection(), DebugLocation - 5.f * -GetGravityDirection() + Velocity,
			100.f, DebugColor, false, -1.f, (uint8)'\000', 10.f);

		FString DebugText = FString::Printf(TEXT("Velocity: %s (Speed: %.2f) (Max: %.2f)"), *Velocity.ToCompactString(), Velocity.Size(), GetMaxSpeed());
		DrawDebugString(GetWorld(), DebugLocation, DebugText, nullptr, DebugColor, 0.f, true);
	}

	// Acceleration
	{
		const FColor DebugColor = FColor::Yellow;
		HeightOffset += OffsetPerElement;
		const float MaxAccelerationLineLength = 200.f;
		const float CurrentMaxAccel = GetMaxAcceleration();
		const float CurrentAccelAsPercentOfMaxAccel = CurrentMaxAccel > 0.f ? Acceleration.Size() / CurrentMaxAccel : 1.f;
		const FVector DebugLocation = TopOfCapsule + HeightOffset * -GetGravityDirection();
		DrawDebugDirectionalArrow(GetWorld(), DebugLocation - 5.f * -GetGravityDirection(),
			DebugLocation - 5.f * -GetGravityDirection() + Acceleration.GetSafeNormal(UE_SMALL_NUMBER) * CurrentAccelAsPercentOfMaxAccel * MaxAccelerationLineLength,
			25.f, DebugColor, false, -1.f, (uint8)'\000', 8.f);

		FString DebugText = FString::Printf(TEXT("Acceleration: %s"), *Acceleration.ToCompactString());
		DrawDebugString(GetWorld(), DebugLocation, DebugText, nullptr, DebugColor, 0.f, true);
	}

	// Movement Mode
	{
		const FColor DebugColor = FColor::Blue;
		HeightOffset += OffsetPerElement;
		FVector DebugLocation = TopOfCapsule + HeightOffset * -GetGravityDirection();
		FString DebugText = FString::Printf(TEXT("MovementMode: %s"), *GetMovementName());
		DrawDebugString(GetWorld(), DebugLocation, DebugText, nullptr, DebugColor, 0.f, true);

		if (IsInWater())
		{
			HeightOffset += OffsetPerElement;
			DebugLocation = TopOfCapsule + HeightOffset * -GetGravityDirection();
			DebugText = FString::Printf(TEXT("ImmersionDepth: %.2f"), ImmersionDepth());
			DrawDebugString(GetWorld(), DebugLocation, DebugText, nullptr, DebugColor, 0.f, true);
		}
	}

	// Jump
	{
		const FColor DebugColor = FColor::Blue;
		HeightOffset += OffsetPerElement;
		FVector DebugLocation = TopOfCapsule + HeightOffset * -GetGravityDirection();
		FString DebugText = FString::Printf(TEXT("bIsJumping: %d Count: %d HoldTime: %.2f"), CharacterOwner->bPressedJump, CharacterOwner->JumpCurrentCount, CharacterOwner->JumpKeyHoldTime);
		DrawDebugString(GetWorld(), DebugLocation, DebugText, nullptr, DebugColor, 0.f, true);
	}

	// Root motion (additive)
	if (CurrentRootMotion.HasAdditiveVelocity())
	{
		const FColor DebugColor = FColor::Cyan;
		HeightOffset += OffsetPerElement;
		const FVector DebugLocation = TopOfCapsule + HeightOffset * -GetGravityDirection();

		FVector CurrentAdditiveVelocity(FVector::ZeroVector);
		CurrentRootMotion.AccumulateAdditiveRootMotionVelocity(0.f, *CharacterOwner, *this, CurrentAdditiveVelocity);

		DrawDebugDirectionalArrow(GetWorld(), DebugLocation, DebugLocation + CurrentAdditiveVelocity, 
			100.f, DebugColor, false, -1.f, (uint8)'\000', 10.f);

		FString DebugText = FString::Printf(TEXT("RootMotionAdditiveVelocity: %s (Speed: %.2f)"), 
			*CurrentAdditiveVelocity.ToCompactString(), CurrentAdditiveVelocity.Size());
		DrawDebugString(GetWorld(), DebugLocation + 5.f * -GetGravityDirection(), DebugText, nullptr, DebugColor, 0.f, true);
	}

	// Root motion (override)
	if (CurrentRootMotion.HasOverrideVelocity())
	{
		const FColor DebugColor = FColor::Green;
		HeightOffset += OffsetPerElement;
		const FVector DebugLocation = TopOfCapsule + HeightOffset * -GetGravityDirection();
		FString DebugText = FString::Printf(TEXT("Has Override RootMotion"));
		DrawDebugString(GetWorld(), DebugLocation, DebugText, nullptr, DebugColor, 0.f, true);
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	return HeightOffset;
}

void UCharacterMovementComponent::ForceReplicationUpdate()
{
	if (HasPredictionData_Server())
	{
		GetPredictionData_Server_Character()->LastUpdateTime = GetWorld()->TimeSeconds - 10.f;
	}
}


void UCharacterMovementComponent::ForceClientAdjustment()
{
	ServerLastClientAdjustmentTime = -1.f;
}

FVector UCharacterMovementComponent::ConstrainInputAcceleration(const FVector& InputAcceleration) const
{
	// walking or falling pawns ignore up/down sliding
	const double InputAccelerationDotGravityNormal = FVector::DotProduct(InputAcceleration, -GetGravityDirection());
	if (!FMath::IsNearlyZero(InputAccelerationDotGravityNormal) && (IsMovingOnGround() || IsFalling()))
	{
		return FVector::VectorPlaneProject(InputAcceleration, -GetGravityDirection());
	}

	return InputAcceleration;
}


FVector UCharacterMovementComponent::ScaleInputAcceleration(const FVector& InputAcceleration) const
{
	return GetMaxAcceleration() * InputAcceleration.GetClampedToMaxSize(1.0f);
}


FVector UCharacterMovementComponent::RoundAcceleration(FVector InAccel) const
{
	// Match FVector_NetQuantize10
	return UE::Net::QuantizeVector(10, InAccel);
}

float UCharacterMovementComponent::ComputeAnalogInputModifier() const
{
	const float MaxAccel = GetMaxAcceleration();
	if (Acceleration.SizeSquared() > 0.f && MaxAccel > UE_SMALL_NUMBER)
	{
		return FMath::Clamp<FVector::FReal>(Acceleration.Size() / MaxAccel, 0.f, 1.f);
	}

	return 0.f;
}

float UCharacterMovementComponent::GetAnalogInputModifier() const
{
	return AnalogInputModifier;
}

float UCharacterMovementComponent::GetSimulationTimeStep(float RemainingTime, int32 Iterations) const
{
	static uint32 s_WarningCount = 0;
	if (RemainingTime > MaxSimulationTimeStep)
	{
		if (Iterations < MaxSimulationIterations)
		{
			// Subdivide moves to be no longer than MaxSimulationTimeStep seconds
			RemainingTime = FMath::Min(MaxSimulationTimeStep, RemainingTime * 0.5f);
		}
		else
		{
			// If this is the last iteration, just use all the remaining time. This is usually better than cutting things short, as the simulation won't move far enough otherwise.
			// Print a throttled warning.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if ((s_WarningCount++ < 100) || (GFrameCounter & 15) == 0)
			{
				UE_LOGF(LogCharacterMovement, Warning, "GetSimulationTimeStep() - Max iterations %d hit while remaining time %.6f > MaxSimulationTimeStep (%.3f) for '%ls', movement '%ls'", MaxSimulationIterations, RemainingTime, MaxSimulationTimeStep, *GetNameSafe(CharacterOwner), *GetMovementName());
			}
#endif
		}
	}

	// no less than MIN_TICK_TIME (to avoid potential divide-by-zero during simulation).
	return FMath::Max(MIN_TICK_TIME, RemainingTime);
}

void UCharacterMovementComponent::SmoothCorrection(const FVector& OldLocation, const FQuat& OldRotation, const FVector& NewLocation, const FQuat& NewRotation)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementSmoothCorrection);
	if (!HasValidData())
	{
		return;
	}

	// We shouldn't be running this on a server that is not a listen server.
	checkSlow(GetNetMode() != NM_DedicatedServer);

	// Only client proxies or remote clients on a listen server should run this code.
	const bool bIsSimulatedProxy = (CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy);
	const bool bIsRemoteAutoProxy = (CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy);
	ensure(bIsSimulatedProxy || bIsRemoteAutoProxy);

	// Getting a correction means new data, so smoothing needs to run.
	bNetworkSmoothingComplete = false;

	// Handle selected smoothing mode.
	if (NetworkSmoothingMode == ENetworkSmoothingMode::Disabled || GetNetMode() == NM_Standalone)
	{
		UpdatedComponent->SetWorldLocationAndRotation(NewLocation, NewRotation, false, nullptr, ETeleportType::TeleportPhysics);
		bNetworkSmoothingComplete = true;
	}
	else if (FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character())
	{
		const UWorld* MyWorld = GetWorld();
		if (!ensure(MyWorld != nullptr))
		{
			return;
		}

		// The mesh doesn't move, but the capsule does so we have a new offset.
		FVector NewToOldVector = (OldLocation - NewLocation);
		if (bIsNavWalkingOnServer && FMath::Abs(GetGravitySpaceZ(NewToOldVector)) < NavWalkingFloorDistTolerance)
		{
			// ignore smoothing on Z axis
			// don't modify new location (local simulation result), since it's probably more accurate than server data
			// and shouldn't matter as long as difference is relatively small
			NewToOldVector = ProjectToGravityFloor(NewToOldVector);
		}

		const float DistSq = NewToOldVector.SizeSquared();
		if (DistSq > FMath::Square(ClientData->MaxSmoothNetUpdateDist))
		{
			ClientData->MeshTranslationOffset = (DistSq > FMath::Square(ClientData->NoSmoothNetUpdateDist)) 
				? FVector::ZeroVector 
				: ClientData->MeshTranslationOffset + ClientData->MaxSmoothNetUpdateDist * NewToOldVector.GetSafeNormal();
		}
		else
		{
			ClientData->MeshTranslationOffset = ClientData->MeshTranslationOffset + NewToOldVector;	
		}

		UE_LOGF(LogCharacterNetSmoothing, Verbose, "Proxy %ls SmoothCorrection(%.2f)", *GetNameSafe(CharacterOwner), FMath::Sqrt(DistSq));
		if (NetworkSmoothingMode == ENetworkSmoothingMode::Linear)
		{
			ClientData->OriginalMeshTranslationOffset	= ClientData->MeshTranslationOffset;

			// Remember the current and target rotation, we're going to lerp between them
			ClientData->OriginalMeshRotationOffset		= OldRotation;
			ClientData->MeshRotationOffset				= OldRotation;
			ClientData->MeshRotationTarget				= NewRotation;

			// Move the capsule, but not the mesh.
			// Note: we don't change rotation, we lerp towards it in SmoothClientPosition.
			if (NewLocation != OldLocation)
			{
				const FScopedPreventAttachedComponentMove PreventMeshMove(CharacterOwner->GetMesh());
				UpdatedComponent->SetWorldLocation(NewLocation, false, nullptr, GetTeleportType());
			}
		}
		else
		{
			// Calc rotation needed to keep current world rotation after UpdatedComponent moves.
			// Take difference between where we were rotated before, and where we're going
			ClientData->MeshRotationOffset = (NewRotation.Inverse() * OldRotation) * ClientData->MeshRotationOffset;
			ClientData->MeshRotationTarget = FQuat::Identity;

			const FScopedPreventAttachedComponentMove PreventMeshMove(CharacterOwner->GetMesh());
			UpdatedComponent->SetWorldLocationAndRotation(NewLocation, NewRotation, false, nullptr, GetTeleportType());
		}
	
		//////////////////////////////////////////////////////////////////////////
		// Update smoothing timestamps

		// If running ahead, pull back slightly. This will cause the next delta to seem slightly longer, and cause us to lerp to it slightly slower.
		if (ClientData->SmoothingClientTimeStamp > ClientData->SmoothingServerTimeStamp)
		{
			const double OldClientTimeStamp = ClientData->SmoothingClientTimeStamp;
			ClientData->SmoothingClientTimeStamp = FMath::LerpStable(ClientData->SmoothingServerTimeStamp, OldClientTimeStamp, 0.5);

			UE_LOGF(LogCharacterNetSmoothing, VeryVerbose, "SmoothCorrection: Pull back client from ClientTimeStamp: %.6f to %.6f, ServerTimeStamp: %.6f for %ls",
				OldClientTimeStamp, ClientData->SmoothingClientTimeStamp, ClientData->SmoothingServerTimeStamp, *GetNameSafe(CharacterOwner));
		}

		// Using server timestamp lets us know how much time actually elapsed, regardless of packet lag variance.
		double OldServerTimeStamp = ClientData->SmoothingServerTimeStamp;
		if (bIsSimulatedProxy)
		{
			// This value is normally only updated on the server, however some code paths might try to read it instead of the replicated value so copy it for proxies as well.
			ServerLastTransformUpdateTimeStamp = CharacterOwner->GetReplicatedServerLastTransformUpdateTimeStamp();
		}
		ClientData->SmoothingServerTimeStamp = ServerLastTransformUpdateTimeStamp;

		// Initial update has no delta.
		if (ClientData->LastCorrectionTime == 0)
		{
			ClientData->SmoothingClientTimeStamp = ClientData->SmoothingServerTimeStamp;
			OldServerTimeStamp = ClientData->SmoothingServerTimeStamp;
		}

		// Don't let the client fall too far behind or run ahead of new server time.
		const double ServerDeltaTime = ClientData->SmoothingServerTimeStamp - OldServerTimeStamp;
		const double MaxOffset = ClientData->MaxClientSmoothingDeltaTime;
		const double MinOffset = FMath::Min(double(ClientData->SmoothNetUpdateTime), MaxOffset);
		
		// MaxDelta is the farthest behind we're allowed to be after receiving a new server time.
		const double MaxDelta = FMath::Clamp(ServerDeltaTime * 1.25, MinOffset, MaxOffset);
		ClientData->SmoothingClientTimeStamp = FMath::Clamp(ClientData->SmoothingClientTimeStamp, ClientData->SmoothingServerTimeStamp - MaxDelta, ClientData->SmoothingServerTimeStamp);

		// Compute actual delta between new server timestamp and client simulation.
		ClientData->LastCorrectionDelta = ClientData->SmoothingServerTimeStamp - ClientData->SmoothingClientTimeStamp;
		ClientData->LastCorrectionTime = MyWorld->GetTimeSeconds();

		UE_LOGF(LogCharacterNetSmoothing, VeryVerbose, "SmoothCorrection: WorldTime: %.6f, ServerTimeStamp: %.6f, ClientTimeStamp: %.6f, Delta: %.6f for %ls",
			MyWorld->GetTimeSeconds(), ClientData->SmoothingServerTimeStamp, ClientData->SmoothingClientTimeStamp, ClientData->LastCorrectionDelta, *GetNameSafe(CharacterOwner));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if ( CharacterMovementCVars::NetVisualizeSimulatedCorrections >= 2 )
		{
			const float Radius		= 4.0f;
			const bool	bPersist	= false;
			const float Lifetime	= 10.0f;
			const int32	Sides		= 8;
			const float ArrowSize	= 4.0f;

			const FVector SimulatedLocation = OldLocation;
			const FVector ServerLocation	= NewLocation + FVector( 0, 0, 0.5f );

			const FVector SmoothLocation	= CharacterOwner->GetMesh()->GetComponentLocation() - CharacterOwner->GetBaseTranslationOffset() + FVector( 0, 0, 1.0f );

			//DrawDebugCoordinateSystem( GetWorld(), ServerLocation + FVector( 0, 0, 300.0f ), UpdatedComponent->GetComponentRotation(), 45.0f, bPersist, Lifetime );

			// Draw simulated location
			DrawCircle( GetWorld(), SimulatedLocation, FVector( 1, 0, 0 ), FVector( 0, 1, 0 ), FColor( 255, 0, 0, 255 ), Radius, Sides, bPersist, Lifetime );

			// Draw server (corrected location)
			DrawCircle( GetWorld(), ServerLocation, FVector( 1, 0, 0 ), FVector( 0, 1, 0 ), FColor( 0, 255, 0, 255 ), Radius, Sides, bPersist, Lifetime );
			
			// Draw smooth simulated location
			FRotationMatrix SmoothMatrix( CharacterOwner->GetMesh()->GetComponentRotation() );
			DrawDebugDirectionalArrow( GetWorld(), SmoothLocation, SmoothLocation + SmoothMatrix.GetScaledAxis( EAxis::Y ) * 5, ArrowSize, FColor( 255, 255, 0, 255 ), bPersist, Lifetime );
			DrawCircle( GetWorld(), SmoothLocation, FVector( 1, 0, 0 ), FVector( 0, 1, 0 ), FColor( 0, 0, 255, 255 ), Radius, Sides, bPersist, Lifetime );

			if ( ClientData->LastServerLocation != FVector::ZeroVector )
			{
				// Arrow showing simulated line
				DrawDebugDirectionalArrow( GetWorld(), ClientData->LastServerLocation, SimulatedLocation, ArrowSize, FColor( 255, 0, 0, 255 ), bPersist, Lifetime );
				
				// Arrow showing server line
				DrawDebugDirectionalArrow( GetWorld(), ClientData->LastServerLocation, ServerLocation, ArrowSize, FColor( 0, 255, 0, 255 ), bPersist, Lifetime );
				
				// Arrow showing smooth location plot
				DrawDebugDirectionalArrow( GetWorld(), ClientData->LastSmoothLocation, SmoothLocation, ArrowSize, FColor( 0, 0, 255, 255 ), bPersist, Lifetime );

				// Line showing correction
				DrawDebugDirectionalArrow( GetWorld(), SimulatedLocation, ServerLocation, ArrowSize, FColor( 128, 0, 0, 255 ), bPersist, Lifetime );
	
				// Line showing smooth vector
				DrawDebugDirectionalArrow( GetWorld(), ServerLocation, SmoothLocation, ArrowSize, FColor( 0, 0, 128, 255 ), bPersist, Lifetime );
			}

			ClientData->LastServerLocation = ServerLocation;
			ClientData->LastSmoothLocation = SmoothLocation;
		}
#endif
	}
}

void UCharacterMovementComponent::SmoothClientPosition(float DeltaSeconds)
{
	if (!HasValidData() || NetworkSmoothingMode == ENetworkSmoothingMode::Disabled)
	{
		return;
	}

	// We shouldn't be running this on a server that is not a listen server.
	checkSlow(GetNetMode() != NM_DedicatedServer);
	checkSlow(GetNetMode() != NM_Standalone);

	// Only client proxies or remote clients on a listen server should run this code.
	const bool bIsSimulatedProxy = (CharacterOwner->GetLocalRole() == ROLE_SimulatedProxy);
	const bool bIsRemoteAutoProxy = (CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy);
	if (!ensure(bIsSimulatedProxy || bIsRemoteAutoProxy))
	{
		return;
	}

	SmoothClientPosition_Interpolate(DeltaSeconds);
	SmoothClientPosition_UpdateVisuals();
}


void UCharacterMovementComponent::SmoothClientPosition_Interpolate(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementSmoothClientPosition_Interp);
	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	if (ClientData)
	{
		if (NetworkSmoothingMode == ENetworkSmoothingMode::Linear)
		{
			const UWorld* MyWorld = GetWorld();

			// Increment client position.
			ClientData->SmoothingClientTimeStamp += DeltaSeconds;

			float LerpPercent = 0.f;
			const float LerpLimit = 1.15f;
			const float TargetDelta = ClientData->LastCorrectionDelta;
			if (TargetDelta > UE_SMALL_NUMBER)
			{
				// Don't let the client get too far ahead (happens on spikes). But we do want a buffer for variable network conditions.
				const float MaxClientTimeAheadPercent = 0.15f;
				const float MaxTimeAhead = TargetDelta * MaxClientTimeAheadPercent;
				ClientData->SmoothingClientTimeStamp = FMath::Min<float>(ClientData->SmoothingClientTimeStamp, ClientData->SmoothingServerTimeStamp + MaxTimeAhead);

				// Compute interpolation alpha based on our client position within the server delta. We should take TargetDelta seconds to reach alpha of 1.
				const float RemainingTime = ClientData->SmoothingServerTimeStamp - ClientData->SmoothingClientTimeStamp;
				const float CurrentSmoothTime = TargetDelta - RemainingTime;
				LerpPercent = FMath::Clamp(CurrentSmoothTime / TargetDelta, 0.0f, LerpLimit);

				UE_LOGF(LogCharacterNetSmoothing, VeryVerbose, "Interpolate: WorldTime: %.6f, ServerTimeStamp: %.6f, ClientTimeStamp: %.6f, Elapsed: %.6f, Alpha: %.6f for %ls",
					MyWorld->GetTimeSeconds(), ClientData->SmoothingServerTimeStamp, ClientData->SmoothingClientTimeStamp, CurrentSmoothTime, LerpPercent, *GetNameSafe(CharacterOwner));
			}
			else
			{
				LerpPercent = 1.0f;
			}

			if (LerpPercent >= 1.0f - UE_KINDA_SMALL_NUMBER)
			{
				if (Velocity.IsNearlyZero())
				{
					ClientData->MeshTranslationOffset = FVector::ZeroVector;
					ClientData->SmoothingClientTimeStamp = ClientData->SmoothingServerTimeStamp;
					bNetworkSmoothingComplete = true;
				}
				else
				{
					// Allow limited forward prediction.
					ClientData->MeshTranslationOffset = FMath::LerpStable(ClientData->OriginalMeshTranslationOffset, FVector::ZeroVector, LerpPercent);
					bNetworkSmoothingComplete = (LerpPercent >= LerpLimit);
				}

				ClientData->MeshRotationOffset = ClientData->MeshRotationTarget;
			}
			else
			{
				ClientData->MeshTranslationOffset = FMath::LerpStable(ClientData->OriginalMeshTranslationOffset, FVector::ZeroVector, LerpPercent);
				ClientData->MeshRotationOffset = FQuat::FastLerp(ClientData->OriginalMeshRotationOffset, ClientData->MeshRotationTarget, LerpPercent).GetNormalized();
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			// Show lerp percent
			if ( CharacterMovementCVars::NetVisualizeSimulatedCorrections >= 1 )
			{
				const FColor DebugColor = FColor::White;
				const FVector DebugLocation = CharacterOwner->GetMesh()->GetComponentLocation() + 130.0f * -GetGravityDirection() - CharacterOwner->GetBaseTranslationOffset();
				FString DebugText = FString::Printf( TEXT( "Lerp: %2.2f" ), LerpPercent );
				DrawDebugString( GetWorld(), DebugLocation, DebugText, nullptr, DebugColor, 0.f, true );
				FString TimeText = FString::Printf( TEXT("ClientTime: %2.2f ServerTime: %2.2f" ), ClientData->SmoothingClientTimeStamp, ClientData->SmoothingServerTimeStamp);
				DrawDebugString( GetWorld(), DebugLocation + 25.f, TimeText, nullptr, DebugColor, 0.f, true);
			}
#endif
		}
		else if (NetworkSmoothingMode == ENetworkSmoothingMode::Exponential)
		{
			// Smooth interpolation of mesh translation to avoid popping of other client pawns unless under a low tick rate.
			// Faster interpolation if stopped.
			const float SmoothLocationTime = Velocity.IsZero() ? 0.5f*ClientData->SmoothNetUpdateTime : ClientData->SmoothNetUpdateTime;
			if (DeltaSeconds < SmoothLocationTime)
			{
				// Slowly decay translation offset
				ClientData->MeshTranslationOffset = (ClientData->MeshTranslationOffset * (1.f - DeltaSeconds / SmoothLocationTime));
			}
			else
			{
				ClientData->MeshTranslationOffset = FVector::ZeroVector;
			}

			// Smooth rotation
			const FQuat MeshRotationTarget = ClientData->MeshRotationTarget;
			if (DeltaSeconds < ClientData->SmoothNetUpdateRotationTime)
			{
				// Slowly decay rotation offset
				ClientData->MeshRotationOffset = FQuat::FastLerp(ClientData->MeshRotationOffset, MeshRotationTarget, DeltaSeconds / ClientData->SmoothNetUpdateRotationTime).GetNormalized();
			}
			else
			{
				ClientData->MeshRotationOffset = MeshRotationTarget;
			}

			// Check if lerp is complete
			if (ClientData->MeshTranslationOffset.IsNearlyZero(1e-2f) && ClientData->MeshRotationOffset.Equals(MeshRotationTarget, 1e-5f))
			{
				bNetworkSmoothingComplete = true;
				// Make sure to snap exactly to target values.
				ClientData->MeshTranslationOffset = FVector::ZeroVector;
				ClientData->MeshRotationOffset = MeshRotationTarget;
			}
		}
		else
		{
			// Unhandled mode
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		//UE_LOGF(LogCharacterNetSmoothing, VeryVerbose, "SmoothClientPosition_Interpolate %ls: Translation: %ls Rotation: %ls",
		//	*GetNameSafe(CharacterOwner), *ClientData->MeshTranslationOffset.ToString(), *ClientData->MeshRotationOffset.ToString());

		if ( CharacterMovementCVars::NetVisualizeSimulatedCorrections >= 1 )
		{
			const FVector DebugLocation = CharacterOwner->GetMesh()->GetComponentLocation() + 300.0f * -GetGravityDirection() - CharacterOwner->GetBaseTranslationOffset();
			DrawDebugBox( GetWorld(), DebugLocation, FVector( 45, 45, 45 ), CharacterOwner->GetMesh()->GetComponentQuat(), FColor( 0, 255, 0 ) );

			//DrawDebugCoordinateSystem( GetWorld(), UpdatedComponent->GetComponentLocation() + FVector( 0, 0, 300.0f ), UpdatedComponent->GetComponentRotation(), 45.0f );
			//DrawDebugBox( GetWorld(), UpdatedComponent->GetComponentLocation() + FVector( 0, 0, 300.0f ), FVector( 45, 45, 45 ), UpdatedComponent->GetComponentQuat(), FColor( 0, 255, 0 ) );

			if ( CharacterMovementCVars::NetVisualizeSimulatedCorrections >= 3 )
			{
				ClientData->SimulatedDebugDrawTime += DeltaSeconds;

				if ( ClientData->SimulatedDebugDrawTime >= 1.0f / 60.0f )
				{
					const float Radius		= 2.0f;
					const bool	bPersist	= false;
					const float Lifetime	= 10.0f;
					const int32	Sides		= 8;

					const FVector SmoothLocation	= CharacterOwner->GetMesh()->GetComponentLocation() - CharacterOwner->GetBaseTranslationOffset();
					const FVector SimulatedLocation	= UpdatedComponent->GetComponentLocation();

					DrawCircle( GetWorld(), SmoothLocation + FVector( 0, 0, 1.5f ), FVector( 1, 0, 0 ), FVector( 0, 1, 0 ), FColor( 0, 0, 255, 255 ), Radius, Sides, bPersist, Lifetime );
					DrawCircle( GetWorld(), SimulatedLocation + FVector( 0, 0, 2.0f ), FVector( 1, 0, 0 ), FVector( 0, 1, 0 ), FColor( 255, 0, 0, 255 ), Radius, Sides, bPersist, Lifetime );

					ClientData->SimulatedDebugDrawTime = 0.0f;
				}
			}
		}
#endif
	}
}

void UCharacterMovementComponent::SmoothClientPosition_UpdateVisuals()
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementSmoothClientPosition_Visual);
	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	USkeletalMeshComponent* Mesh = CharacterOwner->GetMesh();
	if (ClientData && Mesh && !Mesh->IsSimulatingPhysics())
	{
		if (NetworkSmoothingMode == ENetworkSmoothingMode::Linear)
		{
			// Adjust capsule rotation and mesh location. Optimized to trigger only one transform chain update.
			// If we know the rotation is changing that will update children, so it's sufficient to set RelativeLocation directly on the mesh.
			const USceneComponent* MeshParent = Mesh->GetAttachParent();

			FVector MeshParentScale = MeshParent != nullptr ? MeshParent->GetComponentScale() : FVector(1.0f, 1.0f, 1.0f);

			MeshParentScale.X = FMath::IsNearlyZero(MeshParentScale.X) ? 1.0f : MeshParentScale.X;
			MeshParentScale.Y = FMath::IsNearlyZero(MeshParentScale.Y) ? 1.0f : MeshParentScale.Y;
			MeshParentScale.Z = FMath::IsNearlyZero(MeshParentScale.Z) ? 1.0f : MeshParentScale.Z;

			const FVector NewRelLocation = (ClientData->MeshRotationOffset.UnrotateVector(ClientData->MeshTranslationOffset) / MeshParentScale) + CharacterOwner->GetBaseTranslationOffset();
			if (!UpdatedComponent->GetComponentQuat().Equals(ClientData->MeshRotationOffset, 1e-6f))
			{
				const FVector OldLocation = Mesh->GetRelativeLocation();
				const FRotator OldRotation = UpdatedComponent->GetRelativeRotation();
				Mesh->SetRelativeLocation_Direct(NewRelLocation);
				UpdatedComponent->SetWorldRotation(ClientData->MeshRotationOffset);

				// If we did not move from SetWorldRotation, we need to at least call SetRelativeLocation since we were relying on the UpdatedComponent to update the transform of the mesh
				if (UpdatedComponent->GetRelativeRotation() == OldRotation)
				{
					Mesh->SetRelativeLocation_Direct(OldLocation);
					Mesh->SetRelativeLocation(NewRelLocation, false, nullptr, GetTeleportType());
				}
			}
			else
			{
				Mesh->SetRelativeLocation(NewRelLocation, false, nullptr, GetTeleportType());
			}
		}
		else if (NetworkSmoothingMode == ENetworkSmoothingMode::Exponential)
		{
			const USceneComponent* MeshParent = Mesh->GetAttachParent();

			FVector MeshParentScale = MeshParent != nullptr ? MeshParent->GetComponentScale() : FVector(1.0f, 1.0f, 1.0f);

			MeshParentScale.X = FMath::IsNearlyZero(MeshParentScale.X) ? 1.0f : MeshParentScale.X;
			MeshParentScale.Y = FMath::IsNearlyZero(MeshParentScale.Y) ? 1.0f : MeshParentScale.Y;
			MeshParentScale.Z = FMath::IsNearlyZero(MeshParentScale.Z) ? 1.0f : MeshParentScale.Z;

			// Adjust mesh location and rotation
			const FVector NewRelTranslation = (UpdatedComponent->GetComponentToWorld().InverseTransformVectorNoScale(ClientData->MeshTranslationOffset) / MeshParentScale) + CharacterOwner->GetBaseTranslationOffset();
			const FQuat NewRelRotation = ClientData->MeshRotationOffset * CharacterOwner->GetBaseRotationOffset();
			Mesh->SetRelativeLocationAndRotation(NewRelTranslation, NewRelRotation, false, nullptr, GetTeleportType());
		}
		else
		{
			// Unhandled mode
		}
	}
}


bool UCharacterMovementComponent::ClientUpdatePositionAfterServerUpdate()
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementClientUpdatePositionAfterServerUpdate);
	if (!HasValidData())
	{
		return false;
	}

	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	check(ClientData);

	if (!ClientData->bUpdatePosition)
	{
		return false;
	}

	ClientData->bUpdatePosition = false;

	// Don't do any network position updates on things running PHYS_RigidBody
	if (CharacterOwner->GetRootComponent() && CharacterOwner->GetRootComponent()->IsSimulatingPhysics())
	{
		return false;
	}

	if (ClientData->SavedMoves.Num() == 0)
	{
		UE_LOGF(LogNetPlayerMovement, Verbose, "ClientUpdatePositionAfterServerUpdate No saved moves to replay");

		// With no saved moves to resimulate, the move the server updated us with is the last move we've done, no resimulation needed.
		CharacterOwner->bClientResimulateRootMotion = false;
		if (CharacterOwner->bClientResimulateRootMotionSources)
		{
			// With no resimulation, we just update our current root motion to what the server sent us
			UE_LOGF(LogRootMotion, VeryVerbose, "CurrentRootMotion getting updated to ServerUpdate state: %ls", *CharacterOwner->GetName());
			CurrentRootMotion.UpdateStateFrom(CharacterOwner->SavedRootMotion);
			CharacterOwner->bClientResimulateRootMotionSources = false;
		}
		CharacterOwner->SavedRootMotion.Clear();

		return false;
	}

	// Save important values that might get affected by the replay.
	const float SavedAnalogInputModifier = AnalogInputModifier;
	const FRootMotionMovementParams BackupRootMotionParams = RootMotionParams; // For animation root motion
	const FRootMotionSourceGroup BackupRootMotion = CurrentRootMotion;
	const bool bRealPressedJump = CharacterOwner->bPressedJump;
	const float RealJumpMaxHoldTime = CharacterOwner->JumpMaxHoldTime;
	const int32 RealJumpMaxCount = CharacterOwner->JumpMaxCount;
	const bool bRealCrouch = bWantsToCrouch;
	const bool bRealForceMaxAccel = bForceMaxAccel;
	CharacterOwner->bClientWasFalling = (MovementMode == MOVE_Falling);
	CharacterOwner->bClientUpdating = true;
	bForceNextFloorCheck = true;

	// Defer all mesh child updates until all movement completes.
	FScopedMeshMovementUpdate ScopedMeshUpdate(CharacterOwner->GetMesh(), CharacterMovementCVars::bDeferCharacterMeshMovementForAllCorrections);

	// Replay moves that have not yet been acked.
	UE_LOGF(LogNetPlayerMovement, Verbose, "ClientUpdatePositionAfterServerUpdate Replaying %d Moves, starting at Timestamp %f", ClientData->SavedMoves.Num(), ClientData->SavedMoves[0]->TimeStamp);
	for (int32 i=0; i<ClientData->SavedMoves.Num(); i++)
	{
		FSavedMove_Character* const CurrentMove = ClientData->SavedMoves[i].Get();
		checkSlow(CurrentMove != nullptr);

		// Make current SavedMove accessible to any functions that might need it.
		SetCurrentReplayedSavedMove(CurrentMove);

		CurrentMove->PrepMoveFor(CharacterOwner);

		if (ShouldUsePackedMovementRPCs())
		{
			// Make current move data accessible to MoveAutonomous or any other functions that might need it.
			if (FCharacterNetworkMoveData* NewMove = GetNetworkMoveDataContainer().GetNewMoveData())
			{
				SetCurrentNetworkMoveData(NewMove);
				NewMove->ClientFillNetworkMoveData(*CurrentMove, FCharacterNetworkMoveData::ENetworkMoveType::NewMove);
			}
		}

		MoveAutonomous(CurrentMove->TimeStamp, CurrentMove->DeltaTime, CurrentMove->GetCompressedFlags(), CurrentMove->Acceleration);

		CurrentMove->PostUpdate(CharacterOwner, FSavedMove_Character::PostUpdate_Replay);
		SetCurrentNetworkMoveData(nullptr);
		SetCurrentReplayedSavedMove(nullptr);
	}
	const bool bPostReplayPressedJump = CharacterOwner->bPressedJump;

	if (FSavedMove_Character* const PendingMove = ClientData->PendingMove.Get())
	{
		PendingMove->bForceNoCombine = true;
	}

	// Restore saved values.
	AnalogInputModifier = SavedAnalogInputModifier;
	RootMotionParams = BackupRootMotionParams;
	CurrentRootMotion = BackupRootMotion;
	if (CharacterOwner->bClientResimulateRootMotionSources)
	{
		// If we were resimulating root motion sources, it's because we had mismatched state
		// with the server - we just resimulated our SavedMoves and now need to restore
		// CurrentRootMotion with the latest "good state"
		UE_LOGF(LogRootMotion, VeryVerbose, "CurrentRootMotion getting updated after ServerUpdate replays: %ls", *CharacterOwner->GetName());
		CurrentRootMotion.UpdateStateFrom(CharacterOwner->SavedRootMotion);
		CharacterOwner->bClientResimulateRootMotionSources = false;
	}
	CharacterOwner->SavedRootMotion.Clear();
	CharacterOwner->bClientResimulateRootMotion = false;
	CharacterOwner->bClientUpdating = false;
	CharacterOwner->bPressedJump = bRealPressedJump || bPostReplayPressedJump;
	CharacterOwner->JumpMaxHoldTime = RealJumpMaxHoldTime;
	CharacterOwner->JumpMaxCount = RealJumpMaxCount;
	bWantsToCrouch = bRealCrouch;
	bForceMaxAccel = bRealForceMaxAccel;
	bForceNextFloorCheck = true;
	
	return (ClientData->SavedMoves.Num() > 0);
}


bool UCharacterMovementComponent::ForcePositionUpdate(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementForcePositionUpdate);
	if (!HasValidData() || MovementMode == MOVE_None || UpdatedComponent->Mobility != EComponentMobility::Movable)
	{
		return false;
	}

	check(CharacterOwner->GetLocalRole() == ROLE_Authority);
	check(CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy);

	// TODO: this is basically copied from MoveAutonmous, should consolidate. Or consider moving CheckJumpInput inside PerformMovement().
	{
		CharacterOwner->CheckJumpInput(DeltaTime);

		// Acceleration constraints could change after jump input.
		Acceleration = ConstrainInputAcceleration(Acceleration);
		Acceleration = Acceleration.GetClampedToMaxSize(GetMaxAcceleration());
		AnalogInputModifier = ComputeAnalogInputModifier();
	}

	FNetworkPredictionData_Server_Character* ServerData = GetPredictionData_Server_Character();

	if (!ServerData->bForcedUpdateDurationExceeded)
	{
		// Increment client timestamp so we reject client moves after this new simulated time position.
		// See handling of CurrentClientTimeStamp in VerifyClientTimeStamp().
		ServerData->CurrentClientTimeStamp += DeltaTime;
	}

	// Increment server timestamp so ServerLastTransformUpdateTimeStamp gets changed if there is an actual movement.
	const double SavedServerTimestamp = ServerData->ServerAccumulatedClientTimeStamp;
	ServerData->ServerAccumulatedClientTimeStamp += DeltaTime;

	const bool bServerMoveHasOccurred = (ServerData->ServerTimeStampLastServerMove != 0.f);
	if (bServerMoveHasOccurred)
	{
		UE_LOGF(LogNetPlayerMovement, Log, "ForcePositionUpdate: %ls (DeltaTime %.2f -> ServerTimeStamp %.2f) Pawn: %ls", *GetNameSafe(CharacterOwner), DeltaTime, ServerData->CurrentClientTimeStamp, *GetNameSafe(CharacterOwner));
	}

	// Force movement update.
	PerformMovement(DeltaTime);

	// TODO: smooth correction on listen server?
	return true;
}


FNetworkPredictionData_Client* UCharacterMovementComponent::GetPredictionData_Client() const
{
	if (ClientPredictionData == nullptr)
	{
		UCharacterMovementComponent* MutableThis = const_cast<UCharacterMovementComponent*>(this);
		MutableThis->ClientPredictionData = new FNetworkPredictionData_Client_Character(*this);
	}

	return ClientPredictionData;
}

FNetworkPredictionData_Server* UCharacterMovementComponent::GetPredictionData_Server() const
{
	if (ServerPredictionData == nullptr)
	{
		UCharacterMovementComponent* MutableThis = const_cast<UCharacterMovementComponent*>(this);
		MutableThis->ServerPredictionData = new FNetworkPredictionData_Server_Character(*this);
	}

	return ServerPredictionData;
}


FNetworkPredictionData_Client_Character* UCharacterMovementComponent::GetPredictionData_Client_Character() const
{
	// Should only be called on client or listen server (for remote clients) in network games
	checkSlow(CharacterOwner != NULL);
	checkSlow(CharacterOwner->GetLocalRole() < ROLE_Authority || (CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy && GetNetMode() == NM_ListenServer));
	checkSlow(GetNetMode() == NM_Client || GetNetMode() == NM_ListenServer);

	if (ClientPredictionData == nullptr)
	{
		UCharacterMovementComponent* MutableThis = const_cast<UCharacterMovementComponent*>(this);
		MutableThis->ClientPredictionData = static_cast<class FNetworkPredictionData_Client_Character*>(GetPredictionData_Client());
	}

	return ClientPredictionData;
}


FNetworkPredictionData_Server_Character* UCharacterMovementComponent::GetPredictionData_Server_Character() const
{
	// Should only be called on server in network games
	checkSlow(CharacterOwner != NULL);
	checkSlow(CharacterOwner->GetLocalRole() == ROLE_Authority);
	checkSlow(GetNetMode() < NM_Client);

	if (ServerPredictionData == nullptr)
	{
		UCharacterMovementComponent* MutableThis = const_cast<UCharacterMovementComponent*>(this);
		MutableThis->ServerPredictionData = static_cast<class FNetworkPredictionData_Server_Character*>(GetPredictionData_Server());
	}

	return ServerPredictionData;
}


bool UCharacterMovementComponent::HasPredictionData_Client() const
{
	return (ClientPredictionData != nullptr) && HasValidData();
}

bool UCharacterMovementComponent::HasPredictionData_Server() const
{
	return (ServerPredictionData != nullptr) && HasValidData();
}

void UCharacterMovementComponent::ResetPredictionData_Client()
{
	ForceClientAdjustment();
	if (ClientPredictionData)
	{
		delete ClientPredictionData;
		ClientPredictionData = nullptr;
	}
}

void UCharacterMovementComponent::ResetPredictionData_Server()
{
	ForceClientAdjustment();
	if (ServerPredictionData)
	{
		delete ServerPredictionData;
		ServerPredictionData = nullptr;
	}
}

float FNetworkPredictionData_Client_Character::UpdateTimeStampAndDeltaTime(float DeltaTime, class ACharacter & CharacterOwner, class UCharacterMovementComponent & CharacterMovementComponent)
{
	// Reset TimeStamp regularly to combat float accuracy decreasing over time.
	if( CurrentTimeStamp > CharacterMovementComponent.MinTimeBetweenTimeStampResets )
	{
		UE_LOGF(LogNetPlayerMovement, Log, "Resetting Client's TimeStamp %f", CurrentTimeStamp);
		CurrentTimeStamp -= CharacterMovementComponent.MinTimeBetweenTimeStampResets;

		// Mark all buffered moves as having old time stamps, so we make sure to not resend them.
		// That would confuse the server.
		for(int32 MoveIndex=0; MoveIndex<SavedMoves.Num(); MoveIndex++)
		{
			const FSavedMovePtr& CurrentMove = SavedMoves[MoveIndex];
			SavedMoves[MoveIndex]->bOldTimeStampBeforeReset = true;
		}
		// Do LastAckedMove as well. No need to do PendingMove as that move is part of the SavedMoves array.
		if( LastAckedMove.IsValid() )
		{
			LastAckedMove->bOldTimeStampBeforeReset = true;
		}

		// Also apply the reset to any active root motions.
		CharacterMovementComponent.CurrentRootMotion.ApplyTimeStampReset(CharacterMovementComponent.MinTimeBetweenTimeStampResets);
	}

	// Update Current TimeStamp.
	CurrentTimeStamp += DeltaTime;
	float ClientDeltaTime = DeltaTime;

	// Server uses TimeStamps to derive DeltaTime which introduces some rounding errors.
	// Make sure we do the same, so MoveAutonomous uses the same inputs and is deterministic!!
	if( SavedMoves.Num() > 0 )
	{
		const FSavedMovePtr& PreviousMove = SavedMoves.Last();
		if( !PreviousMove->bOldTimeStampBeforeReset )
		{
			// How server will calculate its deltatime to update physics.
			const float ServerDeltaTime = CurrentTimeStamp - PreviousMove->TimeStamp;
			// Have client always use the Server's DeltaTime. Otherwise our physics simulation will differ and we'll trigger too many position corrections and increase our network traffic.
			ClientDeltaTime = ServerDeltaTime;
		}
	}

	return FMath::Min(ClientDeltaTime, MaxMoveDeltaTime * CharacterOwner.GetActorTimeDilation());
}

void UCharacterMovementComponent::ReplicateMoveToServer(float DeltaTime, const FVector& NewAcceleration)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementReplicateMoveToServer);
	check(CharacterOwner != NULL);

	// Can only start sending moves if our controllers are synced up over the network, otherwise we flood the reliable buffer.
	APlayerController* PC = Cast<APlayerController>(CharacterOwner->GetController());
	if (PC && PC->AcknowledgedPawn != CharacterOwner)
	{
		return;
	}

	// Bail out if our character's controller doesn't have a Player. This may be the case when the local player
	// has switched to another controller, such as a debug camera controller.
	if (PC && PC->Player == nullptr)
	{
		return;
	}

	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	if (!ClientData)
	{
		return;
	}
	
	// Update our delta time for physics simulation.
	DeltaTime = ClientData->UpdateTimeStampAndDeltaTime(DeltaTime, *CharacterOwner, *this);

	// Find the oldest (unacknowledged) important move (OldMove).
	// Don't include the last move because it may be combined with the next new move.
	// A saved move is interesting if it differs significantly from the last acknowledged move
	FSavedMovePtr OldMove = NULL;
	if( ClientData->LastAckedMove.IsValid() )
	{
		const int32 NumSavedMoves = ClientData->SavedMoves.Num();
		for (int32 i=0; i < NumSavedMoves-1; i++)
		{
			const FSavedMovePtr& CurrentMove = ClientData->SavedMoves[i];
			if (CurrentMove->IsImportantMove(ClientData->LastAckedMove))
			{
				OldMove = CurrentMove;
				break;
			}
		}
	}

	// Get a SavedMove object to store the movement in.
	FSavedMovePtr NewMovePtr = ClientData->CreateSavedMove();
	FSavedMove_Character* const NewMove = NewMovePtr.Get();
	if (NewMove == nullptr)
	{
		return;
	}

	NewMove->SetMoveFor(CharacterOwner, DeltaTime, NewAcceleration, *ClientData);
	const UWorld* MyWorld = GetWorld();

	// see if the two moves could be combined
	// do not combine moves which have different TimeStamps (before and after reset).
	if (const FSavedMove_Character* PendingMove = ClientData->PendingMove.Get())
	{
		if (PendingMove->CanCombineWith(NewMovePtr, CharacterOwner, ClientData->MaxMoveDeltaTime * CharacterOwner->GetActorTimeDilation(*MyWorld)))
		{
			SCOPE_CYCLE_COUNTER(STAT_CharacterMovementCombineNetMove);

			// Only combine and move back to the start location if we don't move back in to a spot that would make us collide with something new.
			const FVector OldStartLocation = PendingMove->GetRevertedLocation();
			const bool bAttachedToObject = (NewMovePtr->StartAttachParent != nullptr);
			if (bAttachedToObject || !OverlapTest(OldStartLocation, PendingMove->StartRotation.Quaternion(), UpdatedComponent->GetCollisionObjectType(), GetPawnCapsuleCollisionShape(SHRINK_None), CharacterOwner))
			{
				// Avoid updating Mesh bones to physics during the teleport back, since PerformMovement() will update it right away anyway below.
				// Note: this must be before the FScopedMovementUpdate below, since that scope is what actually moves the character and mesh.
				FScopedMeshBoneUpdateOverride ScopedNoMeshBoneUpdate(CharacterOwner->GetMesh(), EKinematicBonesUpdateToPhysics::SkipAllBones);

				// Accumulate multiple transform updates until scope ends.
				FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, true);
				UE_LOGF(LogNetPlayerMovement, VeryVerbose, "CombineMove: add delta %f + %f and revert from %f %f to %f %f", DeltaTime, PendingMove->DeltaTime, UpdatedComponent->GetComponentLocation().X, UpdatedComponent->GetComponentLocation().Y, OldStartLocation.X, OldStartLocation.Y);

				NewMove->CombineWith(PendingMove, CharacterOwner, PC, OldStartLocation);

				if (PC)
				{
					// We reverted position to that at the start of the pending move (above), however some code paths expect rotation to be set correctly
					// before character movement occurs (via FaceRotation), so try that now. The bOrientRotationToMovement path happens later as part of PerformMovement() and PhysicsRotation().
					CharacterOwner->FaceRotation(PC->GetControlRotation(), NewMove->DeltaTime);
				}

				SaveBaseLocation();
				NewMove->SetInitialPosition(CharacterOwner);

				// Remove pending move from move list. It would have to be the last move on the list.
				if (ClientData->SavedMoves.Num() > 0 && ClientData->SavedMoves.Last() == ClientData->PendingMove)
				{
					ClientData->SavedMoves.Pop(EAllowShrinking::No);
				}
				ClientData->FreeMove(ClientData->PendingMove);
				ClientData->PendingMove = nullptr;
				PendingMove = nullptr; // Avoid dangling reference, it's deleted above.
			}
			else
			{
				UE_LOGF(LogNetPlayerMovement, Verbose, "Not combining move [would collide at start location]");
			}
		}
		else
		{
			UE_LOGF(LogNetPlayerMovement, Verbose, "Not combining move [not allowed by CanCombineWith()]");
		}
	}

	// Acceleration should match what we send to the server, plus any other restrictions the server also enforces (see MoveAutonomous).
	Acceleration = NewMove->Acceleration.GetClampedToMaxSize(GetMaxAcceleration());
	AnalogInputModifier = ComputeAnalogInputModifier(); // recompute since acceleration may have changed.

	// Perform the move locally
	CharacterOwner->ClientRootMotionParams.Clear();
	CharacterOwner->SavedRootMotion.Clear();
	PerformMovement(NewMove->DeltaTime);

	NewMove->PostUpdate(CharacterOwner, FSavedMove_Character::PostUpdate_Record);

	// Add NewMove to the list
	if (CharacterOwner->IsReplicatingMovement())
	{
		check(NewMove == NewMovePtr.Get());
		ClientData->SavedMoves.Push(NewMovePtr);

		const bool bCanDelayMove = (CharacterMovementCVars::NetEnableMoveCombining != 0) && CanDelaySendingMove(NewMovePtr);
		
		if (bCanDelayMove && ClientData->PendingMove.IsValid() == false)
		{
			// Decide whether to hold off on move
			const float NetMoveDeltaSeconds = FMath::Clamp(GetClientNetSendDeltaTime(PC, ClientData, NewMovePtr), 1.f/120.f, 1.f/5.f);
			const float SecondsSinceLastMoveSent = MyWorld->GetRealTimeSeconds() - ClientData->ClientUpdateRealTime;

			if (SecondsSinceLastMoveSent < NetMoveDeltaSeconds)
			{
				// Delay sending this move.
				ClientData->PendingMove = NewMovePtr;
				return;
			}
		}

		ClientData->ClientUpdateRealTime = MyWorld->GetRealTimeSeconds();

		UE_CLOGF(CharacterOwner && UpdatedComponent, LogNetPlayerMovement, VeryVerbose, "ClientMove Time %f Acceleration %ls Velocity %ls Position %ls Rotation %ls DeltaTime %f Mode %ls MovementBase %ls.%ls (Dynamic:%d) DualMove? %d",
			NewMove->TimeStamp, *NewMove->Acceleration.ToString(), *Velocity.ToString(), *UpdatedComponent->GetComponentLocation().ToString(), *UpdatedComponent->GetComponentRotation().ToCompactString(), NewMove->DeltaTime, *GetMovementName(),
			*GetNameSafe(NewMove->EndMovementBaseInterfaceData.GetMovementBaseObject()), *NewMove->EndBoneName.ToString(),
			MovementBaseUtility::IsDynamicBase(&NewMove->EndMovementBaseInterfaceData) ? 1 : 0, ClientData->PendingMove.IsValid() ? 1 : 0);

		
		bool bSendServerMove = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Testing options: Simulated packet loss to server
		const float TimeSinceLossStart = (MyWorld->RealTimeSeconds - ClientData->DebugForcedPacketLossTimerStart);
		if (ClientData->DebugForcedPacketLossTimerStart > 0.f && (TimeSinceLossStart < CharacterMovementCVars::NetForceClientServerMoveLossDuration))
		{
			bSendServerMove = false;
			UE_LOGF(LogNetPlayerMovement, Log, "Drop ServerMove, %.2f time remains", CharacterMovementCVars::NetForceClientServerMoveLossDuration - TimeSinceLossStart);
		}
		else if (CharacterMovementCVars::NetForceClientServerMoveLossPercent != 0.f && (RandomStream.FRand() < CharacterMovementCVars::NetForceClientServerMoveLossPercent))
		{
			bSendServerMove = false;
			ClientData->DebugForcedPacketLossTimerStart = (CharacterMovementCVars::NetForceClientServerMoveLossDuration > 0) ? MyWorld->RealTimeSeconds : 0.0f;
			UE_LOGF(LogNetPlayerMovement, Log, "Drop ServerMove, %.2f time remains", CharacterMovementCVars::NetForceClientServerMoveLossDuration);
		}
		else
		{
			ClientData->DebugForcedPacketLossTimerStart = 0.f;
		}
#endif

		// Send move to server if this character is replicating movement
		if (bSendServerMove)
		{
			SCOPE_CYCLE_COUNTER(STAT_CharacterMovementCallServerMove);
			if (ShouldUsePackedMovementRPCs())
			{
				CallServerMovePacked(NewMove, ClientData->PendingMove.Get(), OldMove.Get());
			}
			else
			{
				CallServerMove(NewMove, OldMove.Get());
			}
		}
	}

	ClientData->PendingMove = NULL;
}

namespace UE::Private
{

static UIrisObjectReferencePackageMap* GetIrisPackageMapToCaptureReferences(UNetConnection* NetConnection, FCharacterNetworkSerializationPackedBits& PackedBits)
{
	using namespace UE::Net;

	if (const UEngineReplicationBridge* Bridge = FReplicationSystemUtil::GetActorReplicationBridge(NetConnection))
	{
		if (UIrisObjectReferencePackageMap* ObjectReferencePackageMap = Bridge->GetObjectReferencePackageMap())
		{
			ObjectReferencePackageMap->InitForWrite(&PackedBits.PackageMapExports);

			return ObjectReferencePackageMap;
		}
	}

	return nullptr;
}

static UIrisObjectReferencePackageMap* GetIrisPackageMapToReadReferences(const UNetConnection* NetConnection, const FCharacterNetworkSerializationPackedBits& PackedBits)
{
	using namespace UE::Net;
	if (const UEngineReplicationBridge* Bridge = FReplicationSystemUtil::GetActorReplicationBridge(NetConnection))
	{
		if (UIrisObjectReferencePackageMap* ObjectReferencePackageMap = Bridge->GetObjectReferencePackageMap())
		{
			const UReplicationSystem* ReplicationSystem = UE::Net::FReplicationSystemUtil::GetReplicationSystem(NetConnection->GetDriver());
			ObjectReferencePackageMap->InitForRead(&PackedBits.PackageMapExports, ReplicationSystem->GetNetTokenResolveContext(NetConnection->GetConnectionHandle().GetParentConnectionId()));

			return ObjectReferencePackageMap;
		}
	}

	return nullptr;
}

}

void UCharacterMovementComponent::CallServerMovePacked(const FSavedMove_Character* NewMove, const FSavedMove_Character* PendingMove, const FSavedMove_Character* OldMove)
{
	// Get storage container we'll be using and fill it with movement data
	FCharacterNetworkMoveDataContainer& MoveDataContainer = GetNetworkMoveDataContainer();
	MoveDataContainer.ClientFillNetworkMoveData(NewMove, PendingMove, OldMove);

	// Reset bit writer without affecting allocations
	FBitWriterMark BitWriterReset;
	BitWriterReset.Pop(ServerMoveBitWriter);

	// 'static' to avoid reallocation each invocation
	static FCharacterServerMovePackedBits PackedBits;
	UNetConnection* NetConnection = CharacterOwner->GetNetConnection();	

	if (UPackageMap* PackageMap = UE::Private::GetIrisPackageMapToCaptureReferences(NetConnection, PackedBits))
	{
		ServerMoveBitWriter.PackageMap = PackageMap;
	}
	else
	{
		// Extract the net package map used for serializing object references.
		ServerMoveBitWriter.PackageMap = NetConnection ? ToRawPtr(NetConnection->PackageMap) : nullptr;
	}

	if (ServerMoveBitWriter.PackageMap == nullptr)
	{
		UE_LOGF(LogNetPlayerMovement, Error, "CallServerMovePacked: Failed to find a NetConnection/PackageMap for data serialization!");
		return;
	}

	// Reset captured exports stored in PackedBits
	PackedBits.NetTokensPendingExport.Reset();
	UE::Net::FNetTokenExportScope NetTokenExportScope(ServerMoveBitWriter, NetConnection->GetDriver()->GetNetTokenStore(), PackedBits.NetTokensPendingExport, "CallServerMovePacked");

	// Serialize move struct into a bit stream
	if (!MoveDataContainer.Serialize(*this, ServerMoveBitWriter, ServerMoveBitWriter.PackageMap) || ServerMoveBitWriter.IsError())
	{
		UE_LOGF(LogNetPlayerMovement, Error, "CallServerMovePacked: Failed to serialize out movement data!");
		return;
	}

	// Copy bits to our struct that we can NetSerialize to the server.
	PackedBits.DataBits.SetNumUninitialized(ServerMoveBitWriter.GetNumBits());
	
	check(PackedBits.DataBits.Num() >= ServerMoveBitWriter.GetNumBits());
	FMemory::Memcpy(PackedBits.DataBits.GetData(), ServerMoveBitWriter.GetData(), ServerMoveBitWriter.GetNumBytes());

	// Send bits to server!
	ServerMovePacked_ClientSend(PackedBits);

	MarkForClientCameraUpdate();
}


///// DEPRECATED /////
void UCharacterMovementComponent::CallServerMove
	(
	const FSavedMove_Character* NewMove,
	const FSavedMove_Character* OldMove
	)
{
	check(NewMove != nullptr);

	// Compress rotation down to 5 bytes
	uint32 ClientYawPitchINT = 0;
	uint8 ClientRollBYTE = 0;
	NewMove->GetPackedAngles(ClientYawPitchINT, ClientRollBYTE);

	// Determine if we send absolute or relative location
	UPrimitiveComponent* ClientMovementBase = Cast<UPrimitiveComponent>(NewMove->EndMovementBaseInterfaceData.PhysicsObjectOwner.Get());
	
	const FName ClientBaseBone = NewMove->EndBoneName;
	const FVector SendLocation = MovementBaseUtility::UseRelativeLocation(&NewMove->EndMovementBaseInterfaceData) ? NewMove->SavedRelativeLocation : FRepMovement::RebaseOntoZeroOrigin(NewMove->SavedLocation, this);

	// send old move if it exists
	if (OldMove)
	{
		ServerMoveOld(OldMove->TimeStamp, OldMove->Acceleration, OldMove->GetCompressedFlags());
	}

	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	if (const FSavedMove_Character* const PendingMove = ClientData->PendingMove.Get())
	{
		uint32 OldClientYawPitchINT = 0;
		uint8 OldClientRollBYTE = 0;
		ClientData->PendingMove->GetPackedAngles(OldClientYawPitchINT, OldClientRollBYTE);

		// If we delayed a move without root motion, and our new move has root motion, send these through a special function, so the server knows how to process them.
		if ((PendingMove->RootMotionMontage == NULL) && (NewMove->RootMotionMontage != NULL))
		{
			// send two moves simultaneously
			ServerMoveDualHybridRootMotion(
				PendingMove->TimeStamp,
				PendingMove->Acceleration,
				PendingMove->GetCompressedFlags(),
				OldClientYawPitchINT,
				NewMove->TimeStamp,
				NewMove->Acceleration,
				SendLocation,
				NewMove->GetCompressedFlags(),
				ClientRollBYTE,
				ClientYawPitchINT,
				ClientMovementBase,
				ClientBaseBone,
				NewMove->EndPackedMovementMode
				);
		}
		else
		{
			// send two moves simultaneously
			ServerMoveDual(
				PendingMove->TimeStamp,
				PendingMove->Acceleration,
				PendingMove->GetCompressedFlags(),
				OldClientYawPitchINT,
				NewMove->TimeStamp,
				NewMove->Acceleration,
				SendLocation,
				NewMove->GetCompressedFlags(),
				ClientRollBYTE,
				ClientYawPitchINT,
				ClientMovementBase,
				ClientBaseBone,
				NewMove->EndPackedMovementMode
				);
		}
	}
	else
	{
		ServerMove(
			NewMove->TimeStamp,
			NewMove->Acceleration,
			SendLocation,
			NewMove->GetCompressedFlags(),
			ClientRollBYTE,
			ClientYawPitchINT,
			ClientMovementBase,
			ClientBaseBone,
			NewMove->EndPackedMovementMode
			);
	}

	MarkForClientCameraUpdate();
}


///// DEPRECATED /////
void UCharacterMovementComponent::ServerMoveOld_Implementation
	(
	float OldTimeStamp,
	FVector_NetQuantize10 OldAccel,
	uint8 OldMoveFlags
	)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementServerMove);
	CSV_SCOPED_TIMING_STAT(CharacterMovement, CharacterMovementServerMove);

	if (!HasValidData() || !IsActive())
	{
		return;
	}

	FNetworkPredictionData_Server_Character* ServerData = GetPredictionData_Server_Character();
	check(ServerData);

	if( !VerifyClientTimeStamp(OldTimeStamp, *ServerData) )
	{
		UE_LOGF(LogNetPlayerMovement, VeryVerbose, "ServerMoveOld: TimeStamp expired. %f, CurrentTimeStamp: %f, Character: %ls", OldTimeStamp, ServerData->CurrentClientTimeStamp, *GetNameSafe(CharacterOwner));
		return;
	}

	UE_LOGF(LogNetPlayerMovement, Verbose, "Recovered move from OldTimeStamp %f, DeltaTime: %f", OldTimeStamp, OldTimeStamp - ServerData->CurrentClientTimeStamp);

	const UWorld* MyWorld = GetWorld();
	const float DeltaTime = ServerData->GetServerMoveDeltaTime(OldTimeStamp, CharacterOwner->GetActorTimeDilation(*MyWorld));
	if (DeltaTime > 0.f)
	{
		ServerData->CurrentClientTimeStamp = OldTimeStamp;
		ServerData->ServerAccumulatedClientTimeStamp += DeltaTime;
		ServerData->ServerTimeStamp = MyWorld->GetTimeSeconds();
		ServerData->ServerTimeStampLastServerMove = ServerData->ServerTimeStamp;

		MoveAutonomous(OldTimeStamp, DeltaTime, OldMoveFlags, OldAccel);
	}
	else
	{
		UE_LOGF(LogNetPlayerMovement, Warning, "OldTimeStamp(%f) results in zero or negative actual DeltaTime(%f). Theoretical DeltaTime(%f)",
			OldTimeStamp, DeltaTime, OldTimeStamp - ServerData->CurrentClientTimeStamp);
	}
}


///// DEPRECATED /////
void UCharacterMovementComponent::ServerMoveDual_Implementation(
	float TimeStamp0,
	FVector_NetQuantize10 InAccel0,
	uint8 PendingFlags,
	uint32 View0,
	float TimeStamp,
	FVector_NetQuantize10 InAccel,
	FVector_NetQuantize100 ClientLoc,
	uint8 NewFlags,
	uint8 ClientRoll,
	uint32 View,
	UPrimitiveComponent* ClientMovementBase,
	FName ClientBaseBone,
	uint8 ClientMovementMode)
{
	// Optional scoped movement update to combine moves for cheaper performance on the server.
	FScopedCapsuleMovementUpdate ScopedDualMovementUpdate(UpdatedComponent, bEnableServerDualMoveScopedMovementUpdates);

	ServerMove_Implementation(TimeStamp0, InAccel0, FVector(1.f,2.f,3.f), PendingFlags, ClientRoll, View0, ClientMovementBase, ClientBaseBone, ClientMovementMode);
	ServerMove_Implementation(TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementBase, ClientBaseBone, ClientMovementMode);
}

///// DEPRECATED /////
void UCharacterMovementComponent::ServerMoveDualHybridRootMotion_Implementation(
	float TimeStamp0,
	FVector_NetQuantize10 InAccel0,
	uint8 PendingFlags,
	uint32 View0,
	float TimeStamp,
	FVector_NetQuantize10 InAccel,
	FVector_NetQuantize100 ClientLoc,
	uint8 NewFlags,
	uint8 ClientRoll,
	uint32 View,
	UPrimitiveComponent* ClientMovementBase,
	FName ClientBaseBone,
	uint8 ClientMovementMode)
{
	// First move received didn't use root motion, process it as such.
	CharacterOwner->bServerMoveIgnoreRootMotion = CharacterOwner->IsPlayingNetworkedRootMotionMontage();
	ServerMove_Implementation(TimeStamp0, InAccel0, FVector(1.f, 2.f, 3.f), PendingFlags, ClientRoll, View0, ClientMovementBase, ClientBaseBone, ClientMovementMode);
	CharacterOwner->bServerMoveIgnoreRootMotion = false;

	ServerMove_Implementation(TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementBase, ClientBaseBone, ClientMovementMode);
}

bool UCharacterMovementComponent::VerifyClientTimeStamp(float TimeStamp, FNetworkPredictionData_Server_Character& ServerData)
{
	UWorld* World = GetWorld();

	const bool bFirstMoveAfterForcedUpdates = ServerData.bTriggeringForcedUpdates;
	
	bool bTimeStampResetDetected = false;
	bool bNeedsForcedUpdate = false;
	const bool bIsValid = IsClientTimeStampValid(TimeStamp, ServerData, bTimeStampResetDetected);
	if (bIsValid)
	{
		if (bTimeStampResetDetected)
		{
			UE_LOGF(LogNetPlayerMovement, Log, "TimeStamp reset detected. CurrentTimeStamp: %f, new TimeStamp: %f", ServerData.CurrentClientTimeStamp, TimeStamp);
			if (World != nullptr)
			{
				LastTimeStampResetServerTime = World->GetTimeSeconds();
			}
			OnClientTimeStampResetDetected();
			ServerData.CurrentClientTimeStamp -= MinTimeBetweenTimeStampResets;

			// Also apply the reset to any active root motions.
			CurrentRootMotion.ApplyTimeStampReset(MinTimeBetweenTimeStampResets);
		}
		
		if (bFirstMoveAfterForcedUpdates && (ServerData.bForcedUpdateDurationExceeded || CharacterMovementCVars::bNetServerForcePositionUpdateSyncToClient))
		{
			// We have been performing ForcedUpdates because we hadn't received any moves from this connection in a while but we've now received a new move!
			// Let's sync up to this TimeStamp in order to resolve movement desyncs ASAP
			// This will result in this move having a zero DeltaTime (TimeStamp - CurrentClientTimeStamp), so it will perform no movement but it will send a correction,
			// and we should be able to process the next move that arrives, which will advance ServerTimeStamp and unlock the forced updates (see 'APlayerController::TickActor()').
			UE_LOGF(LogNetPlayerMovement, Log, "Received a new move after performing ForcedUpdates. Updating CurrentClientTimeStamp from %f to %f Pawn: %ls", ServerData.CurrentClientTimeStamp, TimeStamp, *GetNameSafe(CharacterOwner));
			ServerData.CurrentClientTimeStamp = TimeStamp;
			if (World != nullptr)
			{
				ServerData.ServerTimeStamp = World->GetTimeSeconds();
			}
		}
		else if (!bTimeStampResetDetected)
		{
			UE_LOGF(LogNetPlayerMovement, VeryVerbose, "TimeStamp %f Accepted! CurrentTimeStamp: %f Pawn: %ls", TimeStamp, ServerData.CurrentClientTimeStamp, *GetNameSafe(CharacterOwner));
			ProcessClientTimeStampForTimeDiscrepancy(TimeStamp, ServerData);
		}
	}
	else
	{
		if (bTimeStampResetDetected)
		{
			UE_LOGF(LogNetPlayerMovement, Log, "TimeStamp expired. Before TimeStamp Reset. CurrentTimeStamp: %f, TimeStamp: %f Pawn: %ls", ServerData.CurrentClientTimeStamp, TimeStamp, *GetNameSafe(CharacterOwner));
		}
		else
		{
			// NOTE: Disabled: can be dangerous, keeping server advancing and client can't catch up. Has been effectively disabled since CL 16194374, but other changes could enable it again.
			//bNeedsForcedUpdate = (TimeStamp <= ServerData.LastReceivedClientTimeStamp);
		}
	}

	ServerData.LastReceivedClientTimeStamp = TimeStamp;
	ServerData.bLastRequestNeedsForcedUpdates = bNeedsForcedUpdate;
	return bIsValid;
}

void UCharacterMovementComponent::ProcessClientTimeStampForTimeDiscrepancy(float ClientTimeStamp, FNetworkPredictionData_Server_Character& ServerData)
{
	// Should only be called on server in network games
	check(CharacterOwner != NULL);
	check(CharacterOwner->GetLocalRole() == ROLE_Authority);
	checkSlow(GetNetMode() < NM_Client);

	// Movement time discrepancy detection and resolution (potentially caused by client speed hacks, time manipulation)
	// Track client reported time deltas through ServerMove RPCs vs actual server time, when error accumulates enough
	// trigger prevention measures where client must "pay back" the time difference
	const bool bServerMoveHasOccurred = ServerData.ServerTimeStampLastServerMove != 0.f;
	const AGameNetworkManager* GameNetworkManager = (const AGameNetworkManager*)(AGameNetworkManager::StaticClass()->GetDefaultObject());
	if (GameNetworkManager != nullptr && GameNetworkManager->bMovementTimeDiscrepancyDetection && bServerMoveHasOccurred)
	{
		const float WorldTimeSeconds = GetWorld()->GetTimeSeconds();
		const float ServerDelta = (WorldTimeSeconds - ServerData.ServerTimeStamp) * CharacterOwner->CustomTimeDilation;
		const float ClientDelta = ClientTimeStamp - ServerData.CurrentClientTimeStamp;
		const float ClientError = ClientDelta - ServerDelta; // Difference between how much time client has ticked since last move vs server

		// Accumulate raw total discrepancy, unfiltered/unbound (for tracking more long-term trends over the lifetime of the CharacterMovementComponent)
		ServerData.LifetimeRawTimeDiscrepancy += ClientError;

		//
		// 1. Determine total effective discrepancy 
		//
		// NewTimeDiscrepancy is bounded and has a DriftAllowance to limit momentary burst packet loss or 
		// low framerate from having significant impacts, which could cause needing multiple seconds worth of 
		// slow-down/speed-up even though it wasn't intentional time manipulation
		float NewTimeDiscrepancy = ServerData.TimeDiscrepancy + ClientError;
		{
			// Apply drift allowance - forgiving percent difference per time for error
			const float DriftAllowance = GameNetworkManager->MovementTimeDiscrepancyDriftAllowance;
			if (DriftAllowance > 0.f)
			{
				if (NewTimeDiscrepancy > 0.f)
				{
					NewTimeDiscrepancy = FMath::Max(NewTimeDiscrepancy - ServerDelta * DriftAllowance, 0.f);
				}
				else
				{
					NewTimeDiscrepancy = FMath::Min(NewTimeDiscrepancy + ServerDelta * DriftAllowance, 0.f);
				}
			}

			// Enforce bounds
			// Never go below MinTimeMargin - ClientError being negative means the client is BEHIND
			// the server (they are going slower).
			NewTimeDiscrepancy = FMath::Max(NewTimeDiscrepancy, GameNetworkManager->MovementTimeDiscrepancyMinTimeMargin);
		}

		// Determine EffectiveClientError, which is error for the currently-being-processed move after 
		// drift allowances/clamping/resolution mode modifications.
		// We need to know how much the current move contributed towards actionable error so that we don't
		// count error that the server never allowed to impact movement to matter
		float EffectiveClientError = ClientError;
		{
			const float NewTimeDiscrepancyRaw = ServerData.TimeDiscrepancy + ClientError;
			if (NewTimeDiscrepancyRaw != 0.f)
			{
				EffectiveClientError = ClientError * (NewTimeDiscrepancy / NewTimeDiscrepancyRaw);
			}
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Per-frame spew of time discrepancy-related values - useful for investigating state of time discrepancy tracking
		if (CharacterMovementCVars::DebugTimeDiscrepancy > 0)
		{
			UE_LOGF(LogNetPlayerMovement, Warning, "TimeDiscrepancyDetection: ClientError: %f, TimeDiscrepancy: %f, LifetimeRawTimeDiscrepancy: %f (Lifetime %f), Resolving: %d, ClientDelta: %f, ServerDelta: %f, ClientTimeStamp: %f",
				ClientError, ServerData.TimeDiscrepancy, ServerData.LifetimeRawTimeDiscrepancy, WorldTimeSeconds - ServerData.WorldCreationTime, ServerData.bResolvingTimeDiscrepancy, ClientDelta, ServerDelta, ClientTimeStamp);
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		//
		// 2. If we were in resolution mode, determine if we still need to be
		//
		ServerData.bResolvingTimeDiscrepancy = ServerData.bResolvingTimeDiscrepancy && (ServerData.TimeDiscrepancy > 0.f);

		//
		// 3. Determine if NewTimeDiscrepancy is significant enough to trigger detection, and if so, trigger resolution if enabled
		//
		if (!ServerData.bResolvingTimeDiscrepancy)
		{
			if (NewTimeDiscrepancy > GameNetworkManager->MovementTimeDiscrepancyMaxTimeMargin)
			{
				// Time discrepancy detected - client timestamp ahead of where the server thinks it should be!

				// Trigger logic for resolving time discrepancies
				if (GameNetworkManager->bMovementTimeDiscrepancyResolution)
				{
					// Trigger Resolution
					ServerData.bResolvingTimeDiscrepancy = true;

					// Transfer calculated error to official TimeDiscrepancy value, which is the time that will be resolved down
					// in this and subsequent moves until it reaches 0 (meaning we equalize the error)
					// Don't include contribution to error for this move, since we are now going to be in resolution mode
					// and the expected client error (though it did help trigger resolution) won't be allowed
					// to increase error this frame
					ServerData.TimeDiscrepancy = (NewTimeDiscrepancy - EffectiveClientError);
				}
				else
				{
					// We're detecting discrepancy but not handling resolving that through movement component.
					// Clear time stamp error accumulated that triggered detection so we start fresh (maybe it was triggered
					// during severe hitches/packet loss/other non-goodness)
					ServerData.TimeDiscrepancy = 0.f;
				}

				// Project-specific resolution (reporting/recording/analytics)
				OnTimeDiscrepancyDetected(NewTimeDiscrepancy, ServerData.LifetimeRawTimeDiscrepancy, WorldTimeSeconds - ServerData.WorldCreationTime, ClientError);
			}
			else
			{
				// When not in resolution mode and still within error tolerances, accrue total discrepancy
				ServerData.TimeDiscrepancy = NewTimeDiscrepancy;
			}
		}

		//
		// 4. If we are actively resolving time discrepancy, we do so by altering the DeltaTime for the current ServerMove
		//
		if (ServerData.bResolvingTimeDiscrepancy)
		{
			// Optionally force client corrections during time discrepancy resolution
			// This is useful when default project movement error checking is lenient or ClientAuthorativePosition is enabled
			// to ensure time discrepancy resolution is enforced
			if (GameNetworkManager->bMovementTimeDiscrepancyForceCorrectionsDuringResolution)
			{
				ServerData.bForceClientUpdate = true;
			}

			// Movement time discrepancy resolution
			// When the server has detected a significant time difference between what the client ServerMove RPCs are reporting
			// and the actual time that has passed on the server (pointing to potential speed hacks/time manipulation by client),
			// we enter a resolution mode where the usual "base delta's off of client's reported timestamps" is clamped down
			// to the server delta since last movement update, so that during resolution we're not allowing further advantage.
			// Out of that ServerDelta-based move delta, we also need the client to "pay back" the time stolen from initial 
			// time discrepancy detection (held in TimeDiscrepancy) at a specified rate (AGameNetworkManager::TimeDiscrepancyResolutionRate) 
			// to equalize movement time passed on client and server before we can consider the discrepancy "resolved"
			const float ServerCurrentTimeStamp = WorldTimeSeconds;
			const float ServerDeltaSinceLastMovementUpdate = (ServerCurrentTimeStamp - ServerData.ServerTimeStamp) * CharacterOwner->CustomTimeDilation;
			const bool bIsFirstServerMoveThisServerTick = ServerDeltaSinceLastMovementUpdate > 0.f;

			// Restrict ServerMoves to server deltas during time discrepancy resolution 
			// (basing moves off of trusted server time, not client timestamp deltas)
			const float BaseDeltaTime = ServerData.GetBaseServerMoveDeltaTime(ClientTimeStamp, CharacterOwner->GetActorTimeDilation());

			if (!bIsFirstServerMoveThisServerTick)
			{
				// Accumulate client deltas for multiple ServerMoves per server tick so that the next server tick
				// can pay back the full amount of that tick and not be bounded by a single small Move delta
				ServerData.TimeDiscrepancyAccumulatedClientDeltasSinceLastServerTick += BaseDeltaTime;
			}

			float ServerBoundDeltaTime = FMath::Min(BaseDeltaTime + ServerData.TimeDiscrepancyAccumulatedClientDeltasSinceLastServerTick, ServerDeltaSinceLastMovementUpdate);
			ServerBoundDeltaTime = FMath::Max(ServerBoundDeltaTime, 0.f); // No negative deltas allowed

			if (bIsFirstServerMoveThisServerTick)
			{
				// The first ServerMove for a server tick has used the accumulated client delta in the ServerBoundDeltaTime
				// calculation above, clear it out for next frame where we have multiple ServerMoves
				ServerData.TimeDiscrepancyAccumulatedClientDeltasSinceLastServerTick = 0.f;
			}

			// Calculate current move DeltaTime and PayBack time based on resolution rate
			const float ResolutionRate = FMath::Clamp(GameNetworkManager->MovementTimeDiscrepancyResolutionRate, 0.f, 1.f);
			float TimeToPayBack = FMath::Min(ServerBoundDeltaTime * ResolutionRate, ServerData.TimeDiscrepancy); // Make sure we only pay back the time we need to
			float DeltaTimeAfterPayback = ServerBoundDeltaTime - TimeToPayBack;

			// Adjust deltas so current move DeltaTime adheres to minimum tick time
			DeltaTimeAfterPayback = FMath::Max(DeltaTimeAfterPayback, UCharacterMovementComponent::MIN_TICK_TIME);
			TimeToPayBack = ServerBoundDeltaTime - DeltaTimeAfterPayback;

			// Output of resolution: an overridden delta time that will be picked up for this ServerMove, and removing the time
			// we paid back by overriding the DeltaTime to TimeDiscrepancy (time needing resolved)
			ServerData.TimeDiscrepancyResolutionMoveDeltaOverride = DeltaTimeAfterPayback;
			ServerData.TimeDiscrepancy -= TimeToPayBack;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			// Per-frame spew of time discrepancy resolution related values - useful for investigating state of time discrepancy tracking
			if (CharacterMovementCVars::DebugTimeDiscrepancy > 1)
			{
				UE_LOGF(LogNetPlayerMovement, Warning, "TimeDiscrepancyResolution: DeltaOverride: %f, TimeToPayBack: %f, BaseDelta: %f, ServerDeltaSinceLastMovementUpdate: %f, TimeDiscrepancyAccumulatedClientDeltasSinceLastServerTick: %f",
					ServerData.TimeDiscrepancyResolutionMoveDeltaOverride, TimeToPayBack, BaseDeltaTime, ServerDeltaSinceLastMovementUpdate, ServerData.TimeDiscrepancyAccumulatedClientDeltasSinceLastServerTick);
			}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		}
	}
}

bool UCharacterMovementComponent::IsClientTimeStampValid(float TimeStamp, const FNetworkPredictionData_Server_Character& ServerData, bool& bTimeStampResetDetected) const
{
	if (TimeStamp <= 0.f || !FMath::IsFinite(TimeStamp))
	{
		return false;
	}

	// Very large deltas happen around a TimeStamp reset.
	const float DeltaTimeStamp = (TimeStamp - ServerData.CurrentClientTimeStamp);
	if( FMath::Abs(DeltaTimeStamp) > (MinTimeBetweenTimeStampResets * 0.5f) )
	{
		// Client is resetting TimeStamp to increase accuracy.
		bTimeStampResetDetected = true;
		if( DeltaTimeStamp < 0.f )
		{
			// Validate that elapsed time since last reset is reasonable, otherwise client could be manipulating resets.
			if (GetWorld()->TimeSince(LastTimeStampResetServerTime) < (MinTimeBetweenTimeStampResets * 0.5f))
			{
				// Reset too recently
				return false;
			}
			else
			{
				// TimeStamp accepted with reset
				return true;
			}
		}
		else
		{
			// We already reset the TimeStamp, but we just got an old outdated move before the switch, not valid.
			return false;
		}
	}

	// If TimeStamp is in the past, move is outdated, not valid.
	if( TimeStamp <= ServerData.CurrentClientTimeStamp )
	{
		return false;
	}

	// Precision issues (or reordered timestamps from old moves) can cause very small or zero deltas which cause problems.
	if (DeltaTimeStamp < UCharacterMovementComponent::MIN_TICK_TIME)
	{
		return false;
	}
	
	// TimeStamp valid.
	return true;
}

void UCharacterMovementComponent::OnClientTimeStampResetDetected()
{
}

void UCharacterMovementComponent::OnTimeDiscrepancyDetected(float CurrentTimeDiscrepancy, float LifetimeRawTimeDiscrepancy, float Lifetime, float CurrentMoveError)
{
	UE_LOGF(LogNetPlayerMovement, Verbose, "Movement Time Discrepancy detected between client-reported time and server on character %ls. CurrentTimeDiscrepancy: %f, LifetimeRawTimeDiscrepancy: %f, Lifetime: %f, CurrentMoveError %f", 
		CharacterOwner ? *CharacterOwner->GetHumanReadableName() : TEXT("<UNKNOWN>"), 
		CurrentTimeDiscrepancy, 
		LifetimeRawTimeDiscrepancy, 
		Lifetime,
		CurrentMoveError);
}


bool FCharacterNetworkSerializationPackedBits::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	bool bLocalSuccess = true;
	SavedPackageMap = Map;

	// Array size in bits, using minimal number of bytes to write it out.
	uint32 NumBits = DataBits.Num();
	Ar.SerializeIntPacked(NumBits);

	if (NumBits > static_cast<uint32>(CharacterMovementCVars::NetPackedMovementMaxBits))
	{
		// Protect against bad data that could cause server to allocate way too much memory.
		UE_LOGF(LogNetPlayerMovement, Error, "FCharacterNetworkSerializationPackedBits::NetSerialize: Dropping move due to NumBits (%d) exceeding allowable limit (%d). See NetPackedMovementMaxBits.", NumBits, CharacterMovementCVars::NetPackedMovementMaxBits);
		return false;
	}

	if (Ar.IsLoading())
	{
		DataBits.Init(0, NumBits);
	}
	else if (Ar.IsSaving() && NetTokensPendingExport.Num())
	{
		// As we now support exporting NetTokens from shared serialization and FCharacterNetworkSerializationPackedBits serializes data outside of the normal flow
		// we explicitly capture exports which we needs to be inject during actual serialization.
		if (UE::Net::FNetTokenExportContext* ExportContext = UE::Net::FNetTokenExportContext::GetNetTokenExportContext(Ar))
		{
			ExportContext->AppendNetTokensPendingExport(NetTokensPendingExport);
		}		
	}

	// Array data
	Ar.SerializeBits(DataBits.GetData(), NumBits);

	bOutSuccess = bLocalSuccess;
	return !Ar.IsError();
}

void FCharacterNetworkMoveDataContainer::ClientFillNetworkMoveData(const FSavedMove_Character* ClientNewMove, const FSavedMove_Character* ClientPendingMove, const FSavedMove_Character* ClientOldMove)
{
	bDisableCombinedScopedMove = false;

	if (ensure(ClientNewMove))
	{
		NewMoveData->ClientFillNetworkMoveData(*ClientNewMove, FCharacterNetworkMoveData::ENetworkMoveType::NewMove);
		bDisableCombinedScopedMove |= ClientNewMove->bForceNoCombine;
	}

	bHasPendingMove = (ClientPendingMove != nullptr);
	if (bHasPendingMove)
	{
		PendingMoveData->ClientFillNetworkMoveData(*ClientPendingMove, FCharacterNetworkMoveData::ENetworkMoveType::PendingMove);
		bIsDualHybridRootMotionMove = (ClientPendingMove->RootMotionMontage == nullptr) && (ClientNewMove && ClientNewMove->RootMotionMontage != nullptr);
		bDisableCombinedScopedMove |= ClientPendingMove->bForceNoCombine;
	}
	else
	{
		bIsDualHybridRootMotionMove = false;
	}
	
	bHasOldMove = (ClientOldMove != nullptr);
	if (bHasOldMove)
	{
		OldMoveData->ClientFillNetworkMoveData(*ClientOldMove, FCharacterNetworkMoveData::ENetworkMoveType::OldMove);
	}
}


bool FCharacterNetworkMoveDataContainer::Serialize(UCharacterMovementComponent& CharacterMovement, FArchive& Ar, UPackageMap* PackageMap)
{
	// We must have data storage initialized. If not, then the storage container wasn't properly initialized.
	check(NewMoveData && PendingMoveData && OldMoveData);

	// Base move always serialized.
	if (!NewMoveData->Serialize(CharacterMovement, Ar, PackageMap, FCharacterNetworkMoveData::ENetworkMoveType::NewMove))
	{
		return false;
	}
		
	// Optional pending dual move
	Ar.SerializeBits(&bHasPendingMove, 1);
	if (bHasPendingMove)
	{
		Ar.SerializeBits(&bIsDualHybridRootMotionMove, 1);
		if (!PendingMoveData->Serialize(CharacterMovement, Ar, PackageMap, FCharacterNetworkMoveData::ENetworkMoveType::PendingMove))
		{
			return false;
		}
	}

	// Optional old move
	Ar.SerializeBits(&bHasOldMove, 1);
	if (bHasOldMove)
	{
		if (!OldMoveData->Serialize(CharacterMovement, Ar, PackageMap, FCharacterNetworkMoveData::ENetworkMoveType::OldMove))
		{
			return false;
		}
	}

	Ar.SerializeBits(&bDisableCombinedScopedMove, 1);

	return !Ar.IsError();
}


void FCharacterNetworkMoveData::ClientFillNetworkMoveData(const FSavedMove_Character& ClientMove, FCharacterNetworkMoveData::ENetworkMoveType MoveType)
{
	NetworkMoveType = MoveType;

	TimeStamp = ClientMove.TimeStamp;
	ControlRotation = ClientMove.SavedControlRotation;
	CompressedMoveFlags = ClientMove.GetCompressedFlags();
	MovementMode = ClientMove.EndPackedMovementMode;

	// Determine if we send absolute or relative location
	const bool bSendBaseRelativeLocation     = MovementBaseUtility::UseRelativeLocation(&ClientMove.EndMovementBaseInterfaceData);
	const bool bSendBaseRelativeAcceleration = CharacterMovementCVars::NetUseBaseRelativeAcceleration && bSendBaseRelativeLocation;

	const FVector SendLocation     = bSendBaseRelativeLocation ? ClientMove.SavedRelativeLocation : FRepMovement::RebaseOntoZeroOrigin(ClientMove.SavedLocation, ClientMove.CharacterOwner->GetCharacterMovement());
	const FVector SendAcceleration = bSendBaseRelativeAcceleration ? ClientMove.SavedRelativeAcceleration : ClientMove.Acceleration;

	Location = SendLocation;
	Acceleration = SendAcceleration;

	if (bSendBaseRelativeLocation || bSendBaseRelativeAcceleration)
	{
		MovementBasePhysicsObjectOwner = ClientMove.EndMovementBaseInterfaceData.PhysicsObjectOwner.Get();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MovementBase = ClientMove.EndBase.Get();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		MovementBaseBoneName = ClientMove.EndBoneName;
	}
	else
	{
		MovementBasePhysicsObjectOwner = nullptr;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MovementBase = nullptr;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		MovementBaseBoneName = NAME_None;
	}
}


bool FCharacterNetworkMoveData::Serialize(UCharacterMovementComponent& CharacterMovement, FArchive& Ar, UPackageMap* PackageMap, FCharacterNetworkMoveData::ENetworkMoveType MoveType)
{
	NetworkMoveType = MoveType;

	bool bLocalSuccess = true;
	const bool bIsSaving = Ar.IsSaving();

	Ar << TimeStamp;

	// TODO: better packing with single bit per component indicating zero/non-zero
	Acceleration.NetSerialize(Ar, PackageMap, bLocalSuccess);

	Location.NetSerialize(Ar, PackageMap, bLocalSuccess);

	// ControlRotation : FRotator handles each component zero/non-zero test; it uses a single signal bit for zero/non-zero, and uses 16 bits per component if non-zero.
	ControlRotation.NetSerialize(Ar, PackageMap, bLocalSuccess);

	SerializeOptionalValue<uint8>(bIsSaving, Ar, CompressedMoveFlags, 0);
		SerializeOptionalValue<TObjectPtr<UObject>>(bIsSaving, Ar, MovementBasePhysicsObjectOwner, nullptr);
		SerializeOptionalValue<FName>(bIsSaving, Ar, MovementBaseBoneName, NAME_None);
		SerializeOptionalValue<uint8>(bIsSaving, Ar, MovementMode, MOVE_Walking);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MovementBase = Cast<UPrimitiveComponent>(MovementBasePhysicsObjectOwner.Get());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	return !Ar.IsError();
}


void UCharacterMovementComponent::ServerMovePacked_ClientSend(const FCharacterServerMovePackedBits& PackedBits)
{
#if CSV_PROFILER_STATS
	if (CharacterMovementCVars::bClientRecordMovePackedRpcStatsToCsv)
	{
		// track the amount of ServerMovePacked RPCs the client sends to the server
		CSV_CUSTOM_STAT(CharacterMovementDetailed, Client_SendServerMovePacked_RPC, 1, ECsvCustomStatOp::Accumulate);

		// track the size of the RPC sent to the server
		CSV_CUSTOM_STAT(CharacterMovementDetailed, Client_SendServerMovePacked_RPC_NumBits, PackedBits.DataBits.Num(), ECsvCustomStatOp::Accumulate);
	}
#endif //CSV_PROFILER_STATS


	// Pass through RPC call to character on server, there is less RPC bandwidth overhead when used on an Actor rather than a Component.
	CharacterOwner->ServerMovePacked(PackedBits);
}

void UCharacterMovementComponent::ServerMovePacked_ServerReceive(const FCharacterServerMovePackedBits& PackedBits)
{
	if (!HasValidData() || !IsActive())
	{
		return;
	}

	const int32 NumBits = PackedBits.DataBits.Num();
	if (NumBits > CharacterMovementCVars::NetPackedMovementMaxBits)
	{
		// Protect against bad data that could cause server to allocate way too much memory.
		UE_LOGF(LogNetPlayerMovement, Error, "ServerMovePacked_ServerReceive (%ls): Dropping move due to NumBits (%d) exceeding allowable limit (%d). See NetPackedMovementMaxBits.", *GetNameSafe(GetOwner()), NumBits, CharacterMovementCVars::NetPackedMovementMaxBits);
		return;
	}

	// Reuse bit reader to avoid allocating memory each time.
	ServerMoveBitReader.SetData((uint8*)PackedBits.DataBits.GetData(), NumBits);

	if (UIrisObjectReferencePackageMap* PackageMap = UE::Private::GetIrisPackageMapToReadReferences(CharacterOwner->GetNetConnection(), PackedBits))
	{
		ServerMoveBitReader.PackageMap = PackageMap;
	}
	else
	{
		ServerMoveBitReader.PackageMap = PackedBits.GetPackageMap();
	}

	if (ServerMoveBitReader.PackageMap == nullptr)
	{
		devCode(UE_LOGF(LogNetPlayerMovement, Error, "ServerMovePacked_ServerReceive: Failed to find PackageMap for data serialization!"));
		return;
	}

	// Deserialize bits to move data struct.
	// We had to wait until now and use the temp bit stream because the RPC doesn't know about the virtual overrides on the possibly custom struct that is our data container.
	FCharacterNetworkMoveDataContainer& MoveDataContainer = GetNetworkMoveDataContainer();
	if (!MoveDataContainer.Serialize(*this, ServerMoveBitReader, ServerMoveBitReader.PackageMap) || ServerMoveBitReader.IsError())
	{
		devCode(UE_LOGF(LogNetPlayerMovement, Error, "ServerMovePacked_ServerReceive: Failed to serialize movement data!"));
		return;
	}

	ServerMove_HandleMoveData(MoveDataContainer);
}

void UCharacterMovementComponent::ServerMove_HandleMoveData(const FCharacterNetworkMoveDataContainer& MoveDataContainer)
{
	// Optional "old move"
	if (MoveDataContainer.bHasOldMove)
	{
		if (FCharacterNetworkMoveData* OldMove = MoveDataContainer.GetOldMoveData())
		{
			SetCurrentNetworkMoveData(OldMove);
			ServerMove_PerformMovement(*OldMove);
		}
	}

	// Optional scoped movement update for dual moves to combine moves for cheaper performance on the server.
	const bool bMoveAllowsScopedDualMove = MoveDataContainer.bHasPendingMove && !MoveDataContainer.bDisableCombinedScopedMove;
	FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, bMoveAllowsScopedDualMove && bEnableServerDualMoveScopedMovementUpdates && bEnableScopedMovementUpdates);

	// Optional pending move as part of "dual move"
	if (MoveDataContainer.bHasPendingMove)
	{
		if (FCharacterNetworkMoveData* PendingMove = MoveDataContainer.GetPendingMoveData())
		{
			CharacterOwner->bServerMoveIgnoreRootMotion = MoveDataContainer.bIsDualHybridRootMotionMove && CharacterOwner->IsPlayingNetworkedRootMotionMontage();
			SetCurrentNetworkMoveData(PendingMove);
			ServerMove_PerformMovement(*PendingMove);
			CharacterOwner->bServerMoveIgnoreRootMotion = false;
		}
	}

	// Final standard move
	if (FCharacterNetworkMoveData* NewMove = MoveDataContainer.GetNewMoveData())
	{
		SetCurrentNetworkMoveData(NewMove);
		ServerMove_PerformMovement(*NewMove);
	}

	SetCurrentNetworkMoveData(nullptr);
}


void UCharacterMovementComponent::ServerMove_PerformMovement(const FCharacterNetworkMoveData& MoveData)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementServerMove);
	CSV_SCOPED_TIMING_STAT(CharacterMovement, CharacterMovementServerMove);

	if (!HasValidData() || !IsActive())
	{
		return;
	}	

	const float ClientTimeStamp = MoveData.TimeStamp;
	
	FVector ClientAccel = MoveData.Acceleration;

	FMovementBaseInterfaceData MovementBaseInterfaceData = MovementBaseUtility::GetMovementBaseDataFromPhysicsOwner(MoveData.MovementBasePhysicsObjectOwner);
	
	// Convert the move's acceleration to worldspace if necessary
	if (CharacterMovementCVars::NetUseBaseRelativeAcceleration && MovementBaseUtility::IsDynamicBase(&MovementBaseInterfaceData))
	{
		MovementBaseUtility::TransformDirectionToWorld(&MovementBaseInterfaceData, MoveData.MovementBaseBoneName, MoveData.Acceleration, ClientAccel);
	}

	const uint8 ClientMoveFlags = MoveData.CompressedMoveFlags;
	const FRotator ClientControlRotation = MoveData.ControlRotation;

	FNetworkPredictionData_Server_Character* ServerData = GetPredictionData_Server_Character();
	check(ServerData);

	if( !VerifyClientTimeStamp(ClientTimeStamp, *ServerData) )
	{
		const float ServerTimeStamp = ServerData->CurrentClientTimeStamp;
		// This is more severe if the timestamp has a large discrepancy and hasn't been recently reset.
		if (ServerTimeStamp > 1.0f && FMath::Abs(ServerTimeStamp - ClientTimeStamp) > CharacterMovementCVars::NetServerMoveTimestampExpiredWarningThreshold)
		{
			UE_LOGF(LogNetPlayerMovement, Warning, "ServerMove: TimeStamp expired: %f, CurrentTimeStamp: %f, Character: %ls", ClientTimeStamp, ServerTimeStamp, *GetNameSafe(CharacterOwner));
		}
		else
		{
			UE_LOGF(LogNetPlayerMovement, Log, "ServerMove: TimeStamp expired: %f, CurrentTimeStamp: %f, Character: %ls", ClientTimeStamp, ServerTimeStamp, *GetNameSafe(CharacterOwner));
		}		
		return;
	}

	bool bServerReadyForClient = true;
	APlayerController* PC = Cast<APlayerController>(CharacterOwner->GetController());
	if (PC)
	{
		bServerReadyForClient = PC->NotifyServerReceivedClientData(CharacterOwner, ClientTimeStamp);
		if (!bServerReadyForClient)
		{
			ClientAccel = FVector::ZeroVector;
		}
	}

	const UWorld* MyWorld = GetWorld();
	const float DeltaTime = ServerData->GetServerMoveDeltaTime(ClientTimeStamp, CharacterOwner->GetActorTimeDilation(*MyWorld));

	// Defer all mesh child updates until all movement completes.
	FScopedMeshMovementUpdate ScopedMeshUpdate(CharacterOwner->GetMesh());

	if (DeltaTime > 0.f)
	{
		ServerData->CurrentClientTimeStamp = ClientTimeStamp;
		ServerData->ServerAccumulatedClientTimeStamp += DeltaTime;
		ServerData->ServerTimeStamp = MyWorld->GetTimeSeconds();
		ServerData->ServerTimeStampLastServerMove = ServerData->ServerTimeStamp;

		if (AController* CharacterController = Cast<AController>(CharacterOwner->GetController()))
		{
			CharacterController->SetControlRotation(ClientControlRotation);
		}

		if (!bServerReadyForClient)
		{
			return;
		}

		// Perform actual movement
		if ((MyWorld->GetWorldSettings()->GetPauserPlayerState() == NULL))
		{
			FScopedCapsuleMovementUpdate ScopedMovementUpdate(UpdatedComponent, bEnableScopedMovementUpdates);
			if (PC)
			{
				PC->UpdateRotation(DeltaTime);
			}

			MoveAutonomous(ClientTimeStamp, DeltaTime, ClientMoveFlags, ClientAccel);
		}

		UE_CLOGF(CharacterOwner && UpdatedComponent, LogNetPlayerMovement, VeryVerbose, "ServerMove Time %f Acceleration %ls Velocity %ls Position %ls Rotation %ls GravityDirection %ls DeltaTime %f Mode %ls MovementBase %ls.%ls (Dynamic:%d)",
			ClientTimeStamp, *ClientAccel.ToString(), *Velocity.ToString(), *UpdatedComponent->GetComponentLocation().ToString(), *UpdatedComponent->GetComponentRotation().ToCompactString(), *GravityDirection.ToCompactString(), DeltaTime, *GetMovementName(),
			*GetNameSafe(MovementBaseInterfaceData.GetMovementBaseObject()), *CharacterOwner->GetBasedMovement().BoneName.ToString(), MovementBaseUtility::IsDynamicBase(&MovementBaseInterfaceData) ? 1 : 0);
	}

	// Validate move only after old and first dual portion, after all moves are completed.
	if (MoveData.NetworkMoveType == FCharacterNetworkMoveData::ENetworkMoveType::NewMove)
	{
		ServerMoveHandleClientError(ClientTimeStamp, DeltaTime, ClientAccel, MoveData.Location, &MovementBaseInterfaceData, MoveData.MovementBaseBoneName, MoveData.MovementMode);
	}
}


///// DEPRECATED /////
void UCharacterMovementComponent::ServerMove(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	// Making a new FMovementBaseInterfaceData struct so we can use non-deprecated functions - we could also just disable deprecation warnings here too, since the function is deprecated itself
	const FMovementBaseInterfaceData MovementBaseInterfaceData(ClientMovementBase);
	if (MovementBaseUtility::IsDynamicBase(&MovementBaseInterfaceData))
	{
		//UE_LOGF(LogCharacterMovement, Log, "ServerMove: base %ls", *ClientMovementBase->GetName());
		CharacterOwner->ServerMove(TimeStamp, InAccel, ClientLoc, CompressedMoveFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
	}
	else
	{
		//UE_LOGF(LogCharacterMovement, Log, "ServerMoveNoBase");
		CharacterOwner->ServerMoveNoBase(TimeStamp, InAccel, ClientLoc, CompressedMoveFlags, ClientRoll, View, ClientMovementMode);
	}
}

///// DEPRECATED /////
void UCharacterMovementComponent::ServerMove_Implementation(
	float TimeStamp,
	FVector_NetQuantize10 InAccel,
	FVector_NetQuantize100 ClientLoc,
	uint8 MoveFlags,
	uint8 ClientRoll,
	uint32 View,
	UPrimitiveComponent* ClientMovementBase,
	FName ClientBaseBoneName,
	uint8 ClientMovementMode)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementServerMove);
	CSV_SCOPED_TIMING_STAT(CharacterMovement, CharacterMovementServerMove);

	if (!HasValidData() || !IsActive())
	{
		return;
	}	

	FNetworkPredictionData_Server_Character* ServerData = GetPredictionData_Server_Character();
	check(ServerData);

	if( !VerifyClientTimeStamp(TimeStamp, *ServerData) )
	{
		const float ServerTimeStamp = ServerData->CurrentClientTimeStamp;
		// This is more severe if the timestamp has a large discrepancy and hasn't been recently reset.
		if (ServerTimeStamp > 1.0f && FMath::Abs(ServerTimeStamp - TimeStamp) > CharacterMovementCVars::NetServerMoveTimestampExpiredWarningThreshold)
		{
			UE_LOGF(LogNetPlayerMovement, Warning, "ServerMove: TimeStamp expired: %f, CurrentTimeStamp: %f, Character: %ls", TimeStamp, ServerTimeStamp, *GetNameSafe(CharacterOwner));
		}
		else
		{
			UE_LOGF(LogNetPlayerMovement, Log, "ServerMove: TimeStamp expired: %f, CurrentTimeStamp: %f, Character: %ls", TimeStamp, ServerTimeStamp, *GetNameSafe(CharacterOwner));
		}		
		return;
	}

	bool bServerReadyForClient = true;
	APlayerController* PC = Cast<APlayerController>(CharacterOwner->GetController());
	if (PC)
	{
		bServerReadyForClient = PC->NotifyServerReceivedClientData(CharacterOwner, TimeStamp);
		if (!bServerReadyForClient)
		{
			InAccel = FVector::ZeroVector;
		}
	}

	// View components
	const uint16 ViewPitch = (View & 65535);
	const uint16 ViewYaw = (View >> 16);
	
	const FVector Accel = InAccel;

	const UWorld* MyWorld = GetWorld();
	const float DeltaTime = ServerData->GetServerMoveDeltaTime(TimeStamp, CharacterOwner->GetActorTimeDilation(*MyWorld));

	ServerData->CurrentClientTimeStamp = TimeStamp;
	ServerData->ServerAccumulatedClientTimeStamp += DeltaTime;
	ServerData->ServerTimeStamp = MyWorld->GetTimeSeconds();
	ServerData->ServerTimeStampLastServerMove = ServerData->ServerTimeStamp;
	FRotator ViewRot;
	ViewRot.Pitch = FRotator::DecompressAxisFromShort(ViewPitch);
	ViewRot.Yaw = FRotator::DecompressAxisFromShort(ViewYaw);
	ViewRot.Roll = FRotator::DecompressAxisFromByte(ClientRoll);

	if (PC)
	{
		PC->SetControlRotation(ViewRot);
	}

	if (!bServerReadyForClient)
	{
		return;
	}

	// Perform actual movement
	if ((MyWorld->GetWorldSettings()->GetPauserPlayerState() == NULL) && (DeltaTime > 0.f))
	{
		if (PC)
		{
			PC->UpdateRotation(DeltaTime);
		}

		MoveAutonomous(TimeStamp, DeltaTime, MoveFlags, Accel);
	}

	FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData_Mutable();
	UE_CLOGF(CharacterOwner && UpdatedComponent, LogNetPlayerMovement, VeryVerbose, "ServerMove Time %f Acceleration %ls Velocity %ls Position %ls Rotation %ls DeltaTime %f Mode %ls MovementBase %ls.%ls (Dynamic:%d)",
			TimeStamp, *Accel.ToString(), *Velocity.ToString(), *UpdatedComponent->GetComponentLocation().ToString(), *UpdatedComponent->GetComponentRotation().ToCompactString(), DeltaTime, *GetMovementName(),
			*GetNameSafe(MovementBaseInterfaceData ? MovementBaseInterfaceData->GetMovementBaseObject() : nullptr), *CharacterOwner->GetBasedMovement().BoneName.ToString(), MovementBaseUtility::IsDynamicBase(MovementBaseInterfaceData) ? 1 : 0);

	ServerMoveHandleClientError(TimeStamp, DeltaTime, Accel, ClientLoc, MovementBaseInterfaceData, ClientBaseBoneName, ClientMovementMode);
}

void UCharacterMovementComponent::ServerMoveHandleClientError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& RelativeClientLocation, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	FMovementBaseInterfaceData ClientMovementBaseInterfaceData(ClientMovementBase);
	ServerMoveHandleClientError(ClientTimeStamp, DeltaTime, Accel, RelativeClientLocation, &ClientMovementBaseInterfaceData, ClientBaseBoneName, ClientMovementMode);
}

void UCharacterMovementComponent::ServerMoveHandleClientError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& RelativeClientLoc, FMovementBaseInterfaceData* ClientMovementBaseInterfaceData, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	if (!ShouldUsePackedMovementRPCs())
	{
		if (RelativeClientLoc == FVector(1.f,2.f,3.f)) // first part of double servermove
		{
			return;
		}
	}

	FNetworkPredictionData_Server_Character* ServerData = GetPredictionData_Server_Character();
	check(ServerData);

	// Don't prevent more recent updates from being sent if received this frame.
	// We're going to send out an update anyway, might as well be the most recent one.
	APlayerController* PC = Cast<APlayerController>(CharacterOwner->GetController());
	if( (ServerData->LastUpdateTime != GetWorld()->TimeSeconds))
	{
		const AGameNetworkManager* GameNetworkManager = (const AGameNetworkManager*)(AGameNetworkManager::StaticClass()->GetDefaultObject());
		if (GameNetworkManager->WithinUpdateDelayBounds(PC, ServerData->LastUpdateTime))
		{
			return;
		}
	}

	// Offset may be relative to base component
	FVector ClientLoc = RelativeClientLoc;
	if (MovementBaseUtility::UseRelativeLocation(ClientMovementBaseInterfaceData))
	{
		MovementBaseUtility::TransformLocationToWorld(ClientMovementBaseInterfaceData, ClientBaseBoneName, RelativeClientLoc, ClientLoc);
	}
	else
	{
		ClientLoc = FRepMovement::RebaseOntoLocalOrigin(ClientLoc, this);
	}

	FVector ServerLoc = UpdatedComponent->GetComponentLocation();

	// Client may send a null movement base when walking on bases with no relative location (to save bandwidth).
	// In this case don't check movement base in error conditions, use the server one (which avoids an error based on differing bases). Position will still be validated.
	if (!MovementBaseUtility::IsMovementBaseDataValid(ClientMovementBaseInterfaceData))
	{
		TEnumAsByte<EMovementMode> NetMovementMode(MOVE_None);
		TEnumAsByte<EMovementMode> NetGroundMode(MOVE_None);
		uint8 NetCustomMode(0);
		UnpackNetworkMovementMode(ClientMovementMode, NetMovementMode, NetCustomMode, NetGroundMode);
		if (NetMovementMode == MOVE_Walking)
		{
			ClientMovementBaseInterfaceData = GetMovementBaseInterfaceData_Mutable();
			ClientBaseBoneName = CharacterOwner->GetBasedMovement().BoneName;
		}
	}

	// If base location is out of sync on server and client, changing base can result in a jarring correction.
	// So in the case that the base has just changed on server or client, server trusts the client (within a threshold)
	FName MovementBaseBoneName = CharacterOwner->GetBasedMovement().BoneName;
	const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
	const bool bServerIsFalling = IsFalling();
	const bool bClientIsFalling = ClientMovementMode == MOVE_Falling;
	const bool bServerJustLanded = bLastServerIsFalling && !bServerIsFalling;
	const bool bClientJustLanded = bLastClientIsFalling && !bClientIsFalling;

	FVector RelativeLocation = ServerLoc;
	FVector RelativeVelocity = Velocity;
	bool bUseLastBase = false;
	bool bFallingWithinAcceptableError = false;

	// Potentially trust the client a little when landing
	const float ClientAuthorityThreshold = CharacterMovementCVars::ClientAuthorityThresholdOnBaseChange;
	const float MaxFallingCorrectionLeash = CharacterMovementCVars::MaxFallingCorrectionLeash;
	const bool bDeferServerCorrectionsWhenFalling = ClientAuthorityThreshold > 0.f || MaxFallingCorrectionLeash > 0.f;
	if (bDeferServerCorrectionsWhenFalling)
	{
		// Teleports and other movement modes mean we should just trust the server like we normally would
		if (bTeleportedSinceLastUpdate || (MovementMode != MOVE_Walking && MovementMode != MOVE_Falling))
		{
			MaxServerClientErrorWhileFalling = 0.f;
			bCanTrustClientOnLanding = false;
		}

		// MaxFallingCorrectionLeash indicates we'll use a variable correction size based on the error on take-off and the direction of movement.
		// ClientAuthorityThreshold is an static client-trusting correction upon landing.
		// If both are set, use the smaller of the two. If only one is set, use that. If neither are set, we wouldn't even be inside this block.
		float MaxLandingCorrection = 0.f;
		if (ClientAuthorityThreshold > 0.f && MaxFallingCorrectionLeash > 0.f)
		{
			MaxLandingCorrection = FMath::Min(ClientAuthorityThreshold, MaxServerClientErrorWhileFalling);
		}
		else
		{
			MaxLandingCorrection = FMath::Max(ClientAuthorityThreshold, MaxServerClientErrorWhileFalling);
		}

		if (bCanTrustClientOnLanding && MaxLandingCorrection > 0.f && (bClientJustLanded || bServerJustLanded))
		{
			// no longer falling; server should trust client up to a point to finish the landing as the client sees it
			const FVector LocDiff = ServerLoc - ClientLoc;

			if (!LocDiff.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
			{
				if (LocDiff.SizeSquared() < FMath::Square(MaxLandingCorrection))
				{
					ServerLoc = ClientLoc;
					UpdatedComponent->MoveComponent(ServerLoc - UpdatedComponent->GetComponentLocation(), UpdatedComponent->GetComponentQuat(), true, nullptr, EMoveComponentFlags::MOVECOMP_NoFlags, ETeleportType::TeleportPhysics);
					bJustTeleported = true;
				}
				else
				{
					const FVector ClampedDiff = LocDiff.GetSafeNormal() * MaxLandingCorrection;
					ServerLoc -= ClampedDiff;
					UpdatedComponent->MoveComponent(ServerLoc - UpdatedComponent->GetComponentLocation(), UpdatedComponent->GetComponentQuat(), true, nullptr, EMoveComponentFlags::MOVECOMP_NoFlags, ETeleportType::TeleportPhysics);
					bJustTeleported = true;
				}
			}

			MaxServerClientErrorWhileFalling = 0.f;
			bCanTrustClientOnLanding = false;
		}

		// Check for lift-off, going from walking to falling.
		// Note that if bStayBasedInAir is enabled we can't rely on the walking to falling transition, instead run logic on the first tick after clearing the MovementBase
		const bool bCanLiftOffFromBase = bStayBasedInAir
			? !MovementBaseUtility::IsMovementBaseDataValid(MovementBaseInterfaceData) && LastMovementBaseInterfaceData.IsValid() // If we keep the base while in air, consider lift-off if base gets set to null and we had a base last tick
			: bLastServerIsWalking; // If walking last tick, we were can consider lift-off logic

		if (bServerIsFalling && bCanLiftOffFromBase && !bTeleportedSinceLastUpdate)
		{
			float ClientForwardFactor = 1.f;
			if (LastMovementBaseInterfaceData.IsValid() && MovementBaseUtility::IsDynamicBase(&LastMovementBaseInterfaceData) && MaxWalkSpeed > UE_KINDA_SMALL_NUMBER)
			{
				const FVector LastBaseVelocity = MovementBaseUtility::GetMovementBaseVelocity(&LastMovementBaseInterfaceData, LastServerMovementBaseBoneName);
				RelativeVelocity = Velocity - LastBaseVelocity;
				const FVector BaseDirection = ProjectToGravityFloor(LastBaseVelocity).GetSafeNormal();
				const FVector RelativeDirection = RelativeVelocity * (1.f / MaxWalkSpeed);

				ClientForwardFactor = FMath::Clamp(FVector::DotProduct(BaseDirection, RelativeDirection), 0.f, 1.f);

				// To improve position syncing, use old base for take-off
				if (MovementBaseUtility::UseRelativeLocation(&LastMovementBaseInterfaceData))
				{
					// Relative Location
					MovementBaseUtility::TransformLocationToLocal(&LastMovementBaseInterfaceData, LastServerMovementBaseBoneName, UpdatedComponent->GetComponentLocation(), RelativeLocation);
					bUseLastBase = true;
				}
			}

			if (ClientAuthorityThreshold > 0.f && ClientForwardFactor < 1.f)
			{
				const float AdjustedClientAuthorityThreshold = ClientAuthorityThreshold * (1.f - ClientForwardFactor);
				const FVector LocDiff = ServerLoc - ClientLoc;

				// Potentially trust the client a little when taking off in the opposite direction to the base (to help not get corrected back onto the base)
				if (!LocDiff.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
				{
					if (LocDiff.SizeSquared() < FMath::Square(AdjustedClientAuthorityThreshold))
					{
						ServerLoc = ClientLoc;
						UpdatedComponent->MoveComponent(ServerLoc - UpdatedComponent->GetComponentLocation(), UpdatedComponent->GetComponentQuat(), true, nullptr, EMoveComponentFlags::MOVECOMP_NoFlags, ETeleportType::TeleportPhysics);
						bJustTeleported = true;
					}
					else
					{
						const FVector ClampedDiff = LocDiff.GetSafeNormal() * AdjustedClientAuthorityThreshold;
						ServerLoc -= ClampedDiff;
						UpdatedComponent->MoveComponent(ServerLoc - UpdatedComponent->GetComponentLocation(), UpdatedComponent->GetComponentQuat(), true, nullptr, EMoveComponentFlags::MOVECOMP_NoFlags, ETeleportType::TeleportPhysics);
						bJustTeleported = true;
					}
				}
			}

			if (ClientForwardFactor < 1.f)
			{
				MaxServerClientErrorWhileFalling = FMath::Min((ServerLoc - ClientLoc).Size() * (1.f - ClientForwardFactor), MaxFallingCorrectionLeash);
				bCanTrustClientOnLanding = true;
			}
			else
			{
				MaxServerClientErrorWhileFalling = 0.f;
				bCanTrustClientOnLanding = false;
			}
		}
		else if (!bServerIsFalling && bCanTrustClientOnLanding)
		{
			MaxServerClientErrorWhileFalling = 0.f;
			bCanTrustClientOnLanding = false;
		}

		if (MaxServerClientErrorWhileFalling > 0.f && (bServerIsFalling || bClientIsFalling))
		{
			const FVector LocDiff = ServerLoc - ClientLoc;
			if (LocDiff.SizeSquared() <= FMath::Square(MaxServerClientErrorWhileFalling))
			{
				ServerLoc = ClientLoc;
				// Still want a velocity update when we first take off
				bFallingWithinAcceptableError = true;
			}
			else
			{
				// Change ServerLoc to be on the edge of the acceptable error rather than doing a full correction.
				// This is not actually changing the server position, but changing it as far as corrections are concerned.
				// This means we're just holding the client on a longer leash while we're falling.
				ServerLoc = ServerLoc - LocDiff.GetSafeNormal() * FMath::Clamp(MaxServerClientErrorWhileFalling - CharacterMovementCVars::MaxFallingCorrectionLeashBuffer, 0.f, MaxServerClientErrorWhileFalling);
			}
		}
	}

	//Need to re-acquire ServerData, as there's a non-zero chance that MoveComponent triggers a delegate which could call ResetPredictionData_Server
	ServerData = GetPredictionData_Server_Character();
	check(ServerData);

	// Compute the client error from the server's position
	// If client has accumulated a noticeable positional error, correct them.
	bNetworkLargeClientCorrection = ServerData->bForceClientUpdate;
	if (ServerData->bForceClientUpdate || (!bFallingWithinAcceptableError && ServerCheckClientError(ClientTimeStamp, DeltaTime, Accel, ClientLoc, RelativeClientLoc, ClientMovementBaseInterfaceData, ClientBaseBoneName, ClientMovementMode)))
	{
		ServerData->PendingAdjustment.NewVel = Velocity;
		ServerData->PendingAdjustment.NewMovementBasePhysicsObjectOwner = MovementBaseInterfaceData ? MovementBaseInterfaceData->PhysicsObjectOwner.Get() : nullptr;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ServerData->PendingAdjustment.NewBase = CharacterOwner->GetBasedMovement().MovementBase;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		ServerData->PendingAdjustment.NewBaseBoneName = MovementBaseBoneName;
		ServerData->PendingAdjustment.NewLoc = FRepMovement::RebaseOntoZeroOrigin(ServerLoc, this);
		ServerData->PendingAdjustment.NewRot = UpdatedComponent->GetComponentRotation();
		ServerData->PendingAdjustment.GravityDirection = GravityDirection;

		ServerData->PendingAdjustment.bBaseRelativePosition = (bDeferServerCorrectionsWhenFalling && bUseLastBase) || MovementBaseUtility::UseRelativeLocation(MovementBaseInterfaceData);
		ServerData->PendingAdjustment.bBaseRelativeVelocity = false;
		
		// Relative location?
		if (ServerData->PendingAdjustment.bBaseRelativePosition)
		{
			if (bDeferServerCorrectionsWhenFalling && bUseLastBase)
			{
				ServerData->PendingAdjustment.NewVel = RelativeVelocity;
				ServerData->PendingAdjustment.NewMovementBasePhysicsObjectOwner = LastMovementBaseInterfaceData.PhysicsObjectOwner.Get();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				ServerData->PendingAdjustment.NewBase = LastServerMovementBase.Get();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				ServerData->PendingAdjustment.NewBaseBoneName = LastServerMovementBaseBoneName;
				ServerData->PendingAdjustment.NewLoc = RelativeLocation;
			}
			else
			{
				ServerData->PendingAdjustment.NewLoc = CharacterOwner->GetBasedMovement().Location;
				if (CharacterMovementCVars::NetUseBaseRelativeVelocity)
				{
					// Store world velocity converted to local space of movement base
					ServerData->PendingAdjustment.bBaseRelativeVelocity = true;
					const FVector CurrentVelocity = ServerData->PendingAdjustment.NewVel;
					MovementBaseUtility::TransformDirectionToLocal(MovementBaseInterfaceData, MovementBaseBoneName, CurrentVelocity, ServerData->PendingAdjustment.NewVel);
				}
			}
			
			// TODO: this could be a relative rotation, but all client corrections ignore rotation right now except the root motion one, which would need to be updated.
			//ServerData->PendingAdjustment.NewRot = CharacterOwner->GetBasedMovement().Rotation;
		}


#if !UE_BUILD_SHIPPING
		if (CharacterMovementCVars::NetShowCorrections != 0)
		{
			const FVector LocDiff = UpdatedComponent->GetComponentLocation() - ClientLoc;
			const UObject* MovementBaseObject = MovementBaseInterfaceData ? MovementBaseInterfaceData->GetMovementBaseObject() : nullptr;
			const FString BaseString = MovementBaseObject ? MovementBaseObject->GetPathName(MovementBaseObject->GetOutermost()) : TEXT("None");
			UE_LOGF(LogNetPlayerMovement, Warning, "*** Server: Error for %ls at Time=%.3f is %3.3f LocDiff(%ls) ClientLoc(%ls) ServerLoc(%ls) Base: %ls Bone: %ls Accel(%ls) Velocity(%ls)",
				*GetNameSafe(CharacterOwner), ClientTimeStamp, LocDiff.Size(), *LocDiff.ToString(), *ClientLoc.ToString(), *UpdatedComponent->GetComponentLocation().ToString(), *BaseString, *ServerData->PendingAdjustment.NewBaseBoneName.ToString(), *Accel.ToString(), *Velocity.ToString());
			const float DebugLifetime = CharacterMovementCVars::NetCorrectionLifetime;
			DrawDebugCapsule(GetWorld(), UpdatedComponent->GetComponentLocation(), CharacterOwner->GetSimpleCollisionHalfHeight(), CharacterOwner->GetSimpleCollisionRadius(), FQuat::Identity, FColor(100, 255, 100), false, DebugLifetime);
			DrawDebugCapsule(GetWorld(), ClientLoc                    , CharacterOwner->GetSimpleCollisionHalfHeight(), CharacterOwner->GetSimpleCollisionRadius(), FQuat::Identity, FColor(255, 100, 100), false, DebugLifetime);
		}
#endif

		ServerData->LastUpdateTime = GetWorld()->TimeSeconds;
		ServerData->PendingAdjustment.DeltaTime = DeltaTime;
		ServerData->PendingAdjustment.TimeStamp = ClientTimeStamp;
		ServerData->PendingAdjustment.bAckGoodMove = false;
		ServerData->PendingAdjustment.MovementMode = PackNetworkMovementMode();

#if USE_SERVER_PERF_COUNTERS
		PerfCountersIncrement(PerfCounter_NumServerMoveCorrections);
#endif
	}
	else
	{
		if (ServerShouldUseAuthoritativePosition(ClientTimeStamp, DeltaTime, Accel, ClientLoc, RelativeClientLoc, ClientMovementBaseInterfaceData, ClientBaseBoneName, ClientMovementMode))
		{
			const FVector LocDiff = UpdatedComponent->GetComponentLocation() - ClientLoc;
			if (!LocDiff.IsZero() || ClientMovementMode != PackNetworkMovementMode() || GetMovementBaseInterfaceData() != ClientMovementBaseInterfaceData || (CharacterOwner && CharacterOwner->GetBasedMovement().BoneName != ClientBaseBoneName))
			{
				// Just set the position. On subsequent moves we will resolve initially overlapping conditions.
				UpdatedComponent->SetWorldLocation(ClientLoc, false);

				// Trust the client's movement mode.
				ApplyNetworkMovementMode(ClientMovementMode);

				// Update base and floor at new location.
				SetBase(ClientMovementBaseInterfaceData, ClientBaseBoneName);
				UpdateFloorFromAdjustment();

				// Even if base has not changed, we need to recompute the relative offsets (since we've moved).
				SaveBaseLocation();

				LastUpdateLocation = UpdatedComponent ? UpdatedComponent->GetComponentLocation() : FVector::ZeroVector;
				LastUpdateRotation = UpdatedComponent ? UpdatedComponent->GetComponentQuat() : FQuat::Identity;
				LastUpdateVelocity = Velocity;
			}
		}

		// acknowledge receipt of this successful servermove()
		ServerData->PendingAdjustment.TimeStamp = ClientTimeStamp;
		ServerData->PendingAdjustment.bAckGoodMove = true;
	}

#if USE_SERVER_PERF_COUNTERS
	PerfCountersIncrement(PerfCounter_NumServerMoves);
#endif

	ServerData->bForceClientUpdate = false;

	LastMovementBaseInterfaceData = MovementBaseInterfaceData;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LastServerMovementBase = CharacterOwner->GetBasedMovement().MovementBase;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	LastServerMovementBaseBoneName = MovementBaseBoneName;
	bLastClientIsFalling = bClientIsFalling;
	bLastServerIsFalling = bServerIsFalling;
	bLastServerIsWalking = MovementMode == MOVE_Walking;
}

bool UCharacterMovementComponent::ServerCheckClientError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	FMovementBaseInterfaceData ClientMovementBaseInterfaceData(ClientMovementBase);
	return ServerCheckClientError(ClientTimeStamp, DeltaTime, Accel, ClientWorldLocation, RelativeClientLocation, &ClientMovementBaseInterfaceData, ClientBaseBoneName, ClientMovementMode);
}

bool UCharacterMovementComponent::ServerCheckClientError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, FMovementBaseInterfaceData* ClientMovementBaseInterfaceData, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	// Check location difference against global setting
	if (!bIgnoreClientMovementErrorChecksAndCorrection)
	{
#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
		{
			const FVector LocDiff = UpdatedComponent->GetComponentLocation() - ClientWorldLocation;
			FString AdjustedDebugString = FString::Printf(TEXT("ServerCheckClientError LocDiff(%.1f) ExceedsAllowablePositionError(%d) TimeStamp(%f)"),
				LocDiff.Size(), GetDefault<AGameNetworkManager>()->ExceedsAllowablePositionError(LocDiff), ClientTimeStamp);
			RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
		}
#endif

		if (ServerExceedsAllowablePositionError(ClientTimeStamp, DeltaTime, Accel, ClientWorldLocation, RelativeClientLocation, ClientMovementBaseInterfaceData, ClientBaseBoneName, ClientMovementMode))
		{
			return true;
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (CharacterMovementCVars::NetForceClientAdjustmentPercent > UE_SMALL_NUMBER)
		{
			if (RandomStream.FRand() < CharacterMovementCVars::NetForceClientAdjustmentPercent)
			{
				UE_LOGF(LogNetPlayerMovement, VeryVerbose, "** ServerCheckClientError forced by p.NetForceClientAdjustmentPercent");
				return true;
			}
		}
#endif
	}
	else
	{
#if !UE_BUILD_SHIPPING
		if (CharacterMovementCVars::NetShowCorrections != 0)
		{
			UE_LOGF(LogNetPlayerMovement, Warning, "*** Server: %ls is set to ignore error checks and corrections.", *GetNameSafe(CharacterOwner));
		}
#endif // !UE_BUILD_SHIPPING
	}

	return false;
}

bool UCharacterMovementComponent::ServerExceedsAllowablePositionError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	FMovementBaseInterfaceData ClientMovementBaseInterfaceData(ClientMovementBase);
	return ServerExceedsAllowablePositionError(ClientTimeStamp, DeltaTime, Accel, ClientWorldLocation, RelativeClientLocation, &ClientMovementBaseInterfaceData, ClientBaseBoneName, ClientMovementMode);
}


bool UCharacterMovementComponent::ServerExceedsAllowablePositionError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, FMovementBaseInterfaceData* ClientMovementBaseInterfaceData, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	// Check for disagreement in movement mode
	const uint8 CurrentPackedMovementMode = PackNetworkMovementMode();
	if (CurrentPackedMovementMode != ClientMovementMode)
	{
		// Consider this a major correction, see SendClientAdjustment()
		bNetworkLargeClientCorrection = true;
		return true;
	}

	const FVector LocDiff = UpdatedComponent->GetComponentLocation() - ClientWorldLocation;	
	const AGameNetworkManager* GameNetworkManager = (const AGameNetworkManager*)(AGameNetworkManager::StaticClass()->GetDefaultObject());
	if (GameNetworkManager->ExceedsAllowablePositionError(LocDiff))
	{
		bNetworkLargeClientCorrection |= (LocDiff.SizeSquared() > FMath::Square(NetworkLargeClientCorrectionDistance));
		return true;
	}

	return false;
}

bool UCharacterMovementComponent::ServerShouldUseAuthoritativePosition(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	FMovementBaseInterfaceData ClientMovementBaseInterfaceData(ClientMovementBase);
	return ServerShouldUseAuthoritativePosition(ClientTimeStamp, DeltaTime, Accel, ClientWorldLocation, RelativeClientLocation, &ClientMovementBaseInterfaceData, ClientBaseBoneName, ClientMovementMode);
}

bool UCharacterMovementComponent::ServerShouldUseAuthoritativePosition(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, FMovementBaseInterfaceData* ClientMovementBaseInterfaceData, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	if (bServerAcceptClientAuthoritativePosition)
	{
		return true;
	}

	const AGameNetworkManager* GameNetworkManager = (const AGameNetworkManager*)(AGameNetworkManager::StaticClass()->GetDefaultObject());
	if (GameNetworkManager->ClientAuthorativePosition)
	{
		return true;
	}

	return false;
}

bool UCharacterMovementComponent::ServerMove_Validate(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 MoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	return true;
}

void UCharacterMovementComponent::ServerMoveDual(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	// Making a new FMovementBaseInterfaceData struct so we can use non-deprecated functions - we could also just disable deprecation warnings here too, since the function is deprecated itself
	const FMovementBaseInterfaceData MovementBaseInterfaceData(ClientMovementBase);
	if (MovementBaseUtility::IsDynamicBase(&MovementBaseInterfaceData))
	{
		//UE_LOGF(LogCharacterMovement, Log, "ServerMoveDual: base %ls", *ClientMovementBase->GetName());
		CharacterOwner->ServerMoveDual(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
	}
	else
	{
		//UE_LOGF(LogCharacterMovement, Log, "ServerMoveDualNoBase");
		CharacterOwner->ServerMoveDualNoBase(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementMode);
	}
}

bool UCharacterMovementComponent::ServerMoveDual_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	return true;
}

void UCharacterMovementComponent::ServerMoveDualHybridRootMotion(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	CharacterOwner->ServerMoveDualHybridRootMotion(TimeStamp0, InAccel0, PendingFlags, View0, TimeStamp, InAccel, ClientLoc, NewFlags, ClientRoll, View, ClientMovementBase, ClientBaseBoneName, ClientMovementMode);
}

bool UCharacterMovementComponent::ServerMoveDualHybridRootMotion_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode)
{
	return true;
}

void UCharacterMovementComponent::ServerMoveOld(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags)
{
	CharacterOwner->ServerMoveOld(OldTimeStamp, OldAccel, OldMoveFlags);
}

bool UCharacterMovementComponent::ServerMoveOld_Validate(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags)
{
	return true;
}

void UCharacterMovementComponent::MoveAutonomous
	(
	float ClientTimeStamp,
	float DeltaTime,
	uint8 CompressedFlags,
	const FVector& NewAccel
	)
{
	if (!HasValidData())
	{
		return;
	}

	UpdateFromCompressedFlags(CompressedFlags);
	CharacterOwner->CheckJumpInput(DeltaTime);

	Acceleration = ConstrainInputAcceleration(NewAccel);
	Acceleration = Acceleration.GetClampedToMaxSize(GetMaxAcceleration());
	AnalogInputModifier = ComputeAnalogInputModifier();
	
	const FVector OldLocation = UpdatedComponent->GetComponentLocation();
	const FQuat OldRotation = UpdatedComponent->GetComponentQuat();

	const bool bWasPlayingRootMotion = CharacterOwner->IsPlayingRootMotion();

	// Defer all mesh child updates until all movement completes.
	FScopedMeshMovementUpdate ScopedMeshUpdate(CharacterOwner->GetMesh());

	PerformMovement(DeltaTime);

	// Check if data is valid as PerformMovement can mark character for pending kill
	if (!HasValidData())
	{
		return;
	}

	// If not playing root motion, tick animations after physics. We do this here to keep events, notifies, states and transitions in sync with client updates.
	if (CharacterOwner && !CharacterOwner->bClientUpdating && !CharacterOwner->IsPlayingRootMotion() && CharacterOwner->GetMesh())
	{
		if (!bWasPlayingRootMotion) // If we were playing root motion before PerformMovement but aren't anymore, we're on the last frame of anim root motion and have already ticked character
		{
			TickCharacterPose(DeltaTime);
		}
		// TODO: SaveBaseLocation() in case tick moves us?

		USkeletalMeshComponent* OwnerMesh = CharacterOwner->GetMesh();
		check(OwnerMesh != nullptr)
		
		// Fallback to old behaviour if needed. Otherwise, event are dispatched in TickCharacterPose().
		if (!CharacterMovementCVars::EnableQueuedAnimEventsOnServer)
		{
			// Revert back to old behavior if wanted/needed.
			if (OwnerMesh->ShouldOnlyTickMontages(DeltaTime) || OwnerMesh->ShouldOnlyTickMontagesAndRefreshBones(DeltaTime))
			{
				OwnerMesh->ConditionallyDispatchQueuedAnimEvents();
			}
		}
	}

	if (CharacterOwner && UpdatedComponent)
	{
		// Smooth local view of remote clients on listen servers
		if (CharacterMovementCVars::NetEnableListenServerSmoothing &&
			CharacterOwner->GetRemoteRole() == ROLE_AutonomousProxy &&
			IsNetMode(NM_ListenServer))
		{
			SmoothCorrection(OldLocation, OldRotation, UpdatedComponent->GetComponentLocation(), UpdatedComponent->GetComponentQuat());
		}
	}
}


void UCharacterMovementComponent::UpdateFloorFromAdjustment()
{
	if (!HasValidData())
	{
		return;
	}

	// If walking, try to update the cached floor so it is current. This is necessary for UpdateBasedMovement() and MoveAlongFloor() to work properly.
	// If base is now NULL, presumably we are no longer walking. If we had a valid floor but don't find one now, we'll likely start falling.
	// Try to keep the floor valid if we are walking without a movement base (but not NavWalking, see SetBase).
	const bool bFindFloorIfOnGround = (CharacterMovementCVars::bRestoreFloorWhenMissingBaseComponent && IsMovingOnGround() && !bIsNavWalkingOnServer);
	if (MovementBaseUtility::IsMovementBaseDataValid(GetMovementBaseInterfaceData()) || bFindFloorIfOnGround)
	{
		FindFloor(UpdatedComponent->GetComponentLocation(), CurrentFloor, false);
	}
	else
	{
		CurrentFloor.Clear();
	}
}


void UCharacterMovementComponent::MoveResponsePacked_ServerSend(const FCharacterMoveResponsePackedBits& PackedBits)
{
	// Pass through RPC call to character on client, there is less RPC bandwidth overhead when used on an Actor rather than a Component.
	CharacterOwner->ClientMoveResponsePacked(PackedBits);
}

void UCharacterMovementComponent::MoveResponsePacked_ClientReceive(const FCharacterMoveResponsePackedBits& PackedBits)
{
#if CSV_PROFILER_STATS
	if (CharacterMovementCVars::bClientRecordMovePackedRpcStatsToCsv)
	{
		// track the amount of MoveResponsePacked RPCs the client receives from the server
		CSV_CUSTOM_STAT(CharacterMovementDetailed, Client_ReceiveMovePackedResponse_RPC, 1, ECsvCustomStatOp::Accumulate);

		// track the size of the RPC data received from the server
		CSV_CUSTOM_STAT(CharacterMovementDetailed, Client_ReceiveMovePackedResponse_RPC_NumBits, PackedBits.DataBits.Num(), ECsvCustomStatOp::Accumulate);
	}
#endif //CSV_PROFILER_STATS

	if (!HasValidData() || !IsActive())
	{
		return;
	}

	const int32 NumBits = PackedBits.DataBits.Num();
	if (!ensureMsgf(NumBits <= CharacterMovementCVars::NetPackedMovementMaxBits, TEXT("MoveResponsePacked_ClientReceive: NumBits (%d) exceeds CharacterMovementCVars::NetPackedMovementMaxBits (%d)"), NumBits, CharacterMovementCVars::NetPackedMovementMaxBits))
	{
		// Protect against bad data that could cause client to allocate way too much memory.
		devCode(UE_LOGF(LogNetPlayerMovement, Error, "MoveResponsePacked_ClientReceive: NumBits (%d) exceeds allowable limit!", NumBits));
		return;
	}

	// Reuse bit reader to avoid allocating memory each time.
	MoveResponseBitReader.SetData((uint8*)PackedBits.DataBits.GetData(), NumBits);

	if (UPackageMap* PackageMap = UE::Private::GetIrisPackageMapToReadReferences(CharacterOwner->GetNetConnection(), PackedBits))
	{
		MoveResponseBitReader.PackageMap = PackageMap;
	}
	else
	{
		MoveResponseBitReader.PackageMap = PackedBits.GetPackageMap();
	}

	if (MoveResponseBitReader.PackageMap == nullptr)
	{
		devCode(UE_LOGF(LogNetPlayerMovement, Error, "MoveResponsePacked_ClientReceive: Failed to find PackageMap for data serialization!"));
		return;
	}

	// Deserialize bits to response data struct.
	// We had to wait until now and use the temp bit stream because the RPC doesn't know about the virtual overrides on the possibly custom struct that is our data container.
	FCharacterMoveResponseDataContainer& ResponseDataContainer = GetMoveResponseDataContainer();
	if (!ResponseDataContainer.Serialize(*this, MoveResponseBitReader, MoveResponseBitReader.PackageMap) || MoveResponseBitReader.IsError())
	{
		devCode(UE_LOGF(LogNetPlayerMovement, Error, "MoveResponsePacked_ClientReceive: Failed to serialize response data!"));
		return;
	}

	ClientHandleMoveResponse(ResponseDataContainer);
}


void UCharacterMovementComponent::ServerSendMoveResponse(const FClientAdjustment& PendingAdjustment)
{
	// Get storage container we'll be using and fill it with movement data
	FCharacterMoveResponseDataContainer& ResponseDataContainer = GetMoveResponseDataContainer();
	ResponseDataContainer.ServerFillResponseData(*this, PendingAdjustment);

	// Reset bit writer without affecting allocations
	FBitWriterMark BitWriterReset;
	BitWriterReset.Pop(MoveResponseBitWriter);

	// 'static' to avoid reallocation each invocation
	static FCharacterMoveResponsePackedBits PackedBits;
	UNetConnection* NetConnection = CharacterOwner->GetNetConnection();	

	// Extract the net package map used for serializing object references.
	if (UPackageMap* PackageMap = UE::Private::GetIrisPackageMapToCaptureReferences(NetConnection, PackedBits))	
	{
		MoveResponseBitWriter.PackageMap = PackageMap;
	}
	else
	{
		MoveResponseBitWriter.PackageMap = NetConnection ? ToRawPtr(NetConnection->PackageMap) : nullptr;
	}

	if (MoveResponseBitWriter.PackageMap == nullptr)
	{
		UE_LOGF(LogNetPlayerMovement, Error, "ServerSendMoveResponse: Failed to find a NetConnection/PackageMap for data serialization!");
		return;
	}

	// Reset NetTokensPendingExport as PackedBits is reused
	PackedBits.NetTokensPendingExport.Reset();
	UE::Net::FNetTokenExportScope NetTokenExportsScope(MoveResponseBitWriter, NetConnection->GetDriver()->GetNetTokenStore(), PackedBits.NetTokensPendingExport, "ServerSendMoveResponse");

	// Serialize move struct into a bit stream
	if (!ResponseDataContainer.Serialize(*this, MoveResponseBitWriter, MoveResponseBitWriter.PackageMap) || MoveResponseBitWriter.IsError())
	{
		UE_LOGF(LogNetPlayerMovement, Error, "ServerSendMoveResponse: Failed to serialize out response data!");
		return;
	}

	// Copy bits to our struct that we can NetSerialize to the client.
	PackedBits.DataBits.SetNumUninitialized(MoveResponseBitWriter.GetNumBits());

	check(PackedBits.DataBits.Num() >= MoveResponseBitWriter.GetNumBits());
	FMemory::Memcpy(PackedBits.DataBits.GetData(), MoveResponseBitWriter.GetData(), MoveResponseBitWriter.GetNumBytes());

	// Send bits to client!
	MoveResponsePacked_ServerSend(PackedBits);
}


void FCharacterMoveResponseDataContainer::ServerFillResponseData(const UCharacterMovementComponent& CharacterMovement, const FClientAdjustment& PendingAdjustment)
{
	bHasBase = false;
	bHasRotation = false;
	bRootMotionMontageCorrection = false;
	bRootMotionSourceCorrection = false;
	RootMotionTrackPosition = -1.0f;

	ClientAdjustment = PendingAdjustment;

	if (!PendingAdjustment.bAckGoodMove)
	{
		bHasRotation = CharacterMovement.ShouldCorrectRotation();
		bHasBase = (PendingAdjustment.NewMovementBasePhysicsObjectOwner != nullptr);
		if (const ACharacter* CharacterOwner = CharacterMovement.GetCharacterOwner())
		{
			bRootMotionMontageCorrection = CharacterOwner->IsPlayingNetworkedRootMotionMontage();
			RootMotionTrackPosition = bRootMotionMontageCorrection ? CharacterOwner->GetRootMotionAnimMontageInstance()->GetPosition() : -1.f;

			const FRotator Rotation = PendingAdjustment.NewRot.GetNormalized();
			RootMotionRotation = FVector_NetQuantizeNormal(Rotation.Pitch / 180.f, Rotation.Yaw / 180.f, Rotation.Roll / 180.f);

			if (CharacterMovement.CurrentRootMotion.HasActiveRootMotionSources())
			{
				// Setting this flag will cause GetRootMotionSourceGroup() to return the correct server verision of the current root motion for serialization.
				bRootMotionSourceCorrection = true;
			}
		}
	}
}

bool FCharacterMoveResponseDataContainer::Serialize(UCharacterMovementComponent& CharacterMovement, FArchive& Ar, UPackageMap* PackageMap)
{
	bool bLocalSuccess = true;
	const bool bIsSaving = Ar.IsSaving();

	Ar.SerializeBits(&ClientAdjustment.bAckGoodMove, 1);
	Ar << ClientAdjustment.TimeStamp;

	if (IsCorrection())
	{
		Ar.SerializeBits(&bHasBase, 1);
		Ar.SerializeBits(&bHasRotation, 1);
		Ar.SerializeBits(&bRootMotionMontageCorrection, 1);
		Ar.SerializeBits(&bRootMotionSourceCorrection, 1);

		ClientAdjustment.NewLoc.NetSerialize(Ar, PackageMap, bLocalSuccess);
		ClientAdjustment.NewVel.NetSerialize(Ar, PackageMap, bLocalSuccess);
		NetSerializeOptionalValue(bIsSaving, Ar, ClientAdjustment.GravityDirection, UCharacterMovementComponent::DefaultGravityDirection, PackageMap);

		if (bHasRotation)
		{
			ClientAdjustment.NewRot.NetSerialize(Ar, PackageMap, bLocalSuccess);
		}
		else if (!bIsSaving)
		{
			ClientAdjustment.NewRot = FRotator::ZeroRotator;
		}

		SerializeOptionalValue<TObjectPtr<UObject>>(bIsSaving, Ar, ClientAdjustment.NewMovementBasePhysicsObjectOwner, nullptr);
		SerializeOptionalValue<FName>(bIsSaving, Ar, ClientAdjustment.NewBaseBoneName, NAME_None);
		SerializeOptionalValue<uint8>(bIsSaving, Ar, ClientAdjustment.MovementMode, MOVE_Walking);
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if UE_WITH_REMOTE_OBJECT_HANDLE
		ClientAdjustment.NewBase = Cast<UPrimitiveComponent>(ClientAdjustment.NewMovementBasePhysicsObjectOwner.Get());
#else
		ClientAdjustment.NewBase = Cast<UPrimitiveComponent>(ClientAdjustment.NewMovementBasePhysicsObjectOwner.Get());
#endif
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		Ar.SerializeBits(&ClientAdjustment.bBaseRelativePosition, 1);
		Ar.SerializeBits(&ClientAdjustment.bBaseRelativeVelocity, 1);

		if (bRootMotionMontageCorrection)
		{
			Ar << RootMotionTrackPosition;
		}
		else if (!bIsSaving)
		{
			RootMotionTrackPosition = -1.0f;
		}

		if (bRootMotionSourceCorrection)
		{
			if (FRootMotionSourceGroup* RootMotionSourceGroup = GetRootMotionSourceGroup(CharacterMovement))
			{
				RootMotionSourceGroup->NetSerialize(Ar, PackageMap, bLocalSuccess, 3 /*MaxNumRootMotionSourcesToSerialize*/);
			}
		}

		if (bRootMotionMontageCorrection || bRootMotionSourceCorrection)
		{
			RootMotionRotation.NetSerialize(Ar, PackageMap, bLocalSuccess);
		}
	}

	return !Ar.IsError();
}

void UCharacterMovementComponent::ClientHandleMoveResponse(const FCharacterMoveResponseDataContainer& MoveResponse)
{
	if (MoveResponse.IsGoodMove())
	{
		ClientAckGoodMove_Implementation(MoveResponse.ClientAdjustment.TimeStamp);
	}
	else
	{
		FMovementBaseInterfaceData MovementBaseInterfaceDataFromClientAdjustment = MovementBaseUtility::GetMovementBaseDataFromPhysicsOwner(MoveResponse.ClientAdjustment.NewMovementBasePhysicsObjectOwner);
		
		// Wrappers to old RPC handlers, to maintain compatibility. If overrides need additional serialized data, they can access GetMoveResponseDataContainer()
		if (MoveResponse.bRootMotionSourceCorrection)
		{
			if (FRootMotionSourceGroup* RootMotionSourceGroup = MoveResponse.GetRootMotionSourceGroup(*this))
			{
				ClientAdjustRootMotionSourcePosition_Implementation(
					MoveResponse.ClientAdjustment.TimeStamp,
					*RootMotionSourceGroup,
					MoveResponse.bRootMotionMontageCorrection,
					MoveResponse.RootMotionTrackPosition,
					MoveResponse.ClientAdjustment.NewLoc,
					MoveResponse.RootMotionRotation,
					GetGravitySpaceZ(MoveResponse.ClientAdjustment.NewVel),
					&MovementBaseInterfaceDataFromClientAdjustment,
					MoveResponse.ClientAdjustment.NewBaseBoneName,
					MoveResponse.bHasBase,
					MoveResponse.ClientAdjustment.bBaseRelativePosition,
					MoveResponse.ClientAdjustment.MovementMode);
			}
		}
		else if (MoveResponse.bRootMotionMontageCorrection)
		{
			ClientAdjustRootMotionPosition_Implementation(
				MoveResponse.ClientAdjustment.TimeStamp,
				MoveResponse.RootMotionTrackPosition,
				MoveResponse.ClientAdjustment.NewLoc,
				MoveResponse.RootMotionRotation,
				GetGravitySpaceZ(MoveResponse.ClientAdjustment.NewVel),
				&MovementBaseInterfaceDataFromClientAdjustment,
				MoveResponse.ClientAdjustment.NewBaseBoneName,
				MoveResponse.bHasBase,
				MoveResponse.ClientAdjustment.bBaseRelativePosition,
				MoveResponse.ClientAdjustment.MovementMode);
		}
		else
		{
			ClientAdjustPosition_Implementation(
				MoveResponse.ClientAdjustment.TimeStamp,
				MoveResponse.ClientAdjustment.NewLoc,
				MoveResponse.ClientAdjustment.NewVel,
				&MovementBaseInterfaceDataFromClientAdjustment,
				MoveResponse.ClientAdjustment.NewBaseBoneName,
				MoveResponse.bHasBase,
				MoveResponse.ClientAdjustment.bBaseRelativePosition,
				MoveResponse.ClientAdjustment.MovementMode,
				MoveResponse.bHasRotation ? MoveResponse.ClientAdjustment.NewRot : TOptional<FRotator>()
				);
		}
	}
}


FRootMotionSourceGroup* FCharacterMoveResponseDataContainer::GetRootMotionSourceGroup(UCharacterMovementComponent& CharacterMovement) const
{
	if (!bRootMotionSourceCorrection)
	{
		return nullptr;
	}

	if (CharacterMovement.GetNetMode() < NM_Client)
	{
		// Servers use current root motion state
		return &CharacterMovement.CurrentRootMotion;
	}
	else
	{
		// Clients use a container for server correction data
		return &CharacterMovement.ServerCorrectionRootMotion;
	}
}


void UCharacterMovementComponent::SendClientAdjustment()
{
	if (!HasValidData())
	{
		return;
	}

	FNetworkPredictionData_Server_Character* ServerData = GetPredictionData_Server_Character();
	check(ServerData);

	if (ServerData->PendingAdjustment.TimeStamp <= 0.f)
	{
		return;
	}

	const float CurrentTime = GetWorld()->GetTimeSeconds();
	if (ServerData->PendingAdjustment.bAckGoodMove)
	{
		// just notify client this move was received
		if (CurrentTime - ServerLastClientGoodMoveAckTime > NetworkMinTimeBetweenClientAckGoodMoves)
		{
			ServerLastClientGoodMoveAckTime = CurrentTime;
			if (ShouldUsePackedMovementRPCs())
			{
				ServerSendMoveResponse(ServerData->PendingAdjustment);
			}
			else
			{
				ClientAckGoodMove(ServerData->PendingAdjustment.TimeStamp);
			}
		}
	}
	else
	{
		// We won't be back in here until the next client move and potential correction is received, so use the correct time now.
		// Protect against bad data by taking appropriate min/max of editable values.
		const float AdjustmentTimeThreshold = bNetworkLargeClientCorrection ?
			FMath::Min(NetworkMinTimeBetweenClientAdjustmentsLargeCorrection, NetworkMinTimeBetweenClientAdjustments) :
			FMath::Max(NetworkMinTimeBetweenClientAdjustmentsLargeCorrection, NetworkMinTimeBetweenClientAdjustments);

		// Check if correction is throttled based on time limit between updates.
		if (CurrentTime - ServerLastClientAdjustmentTime > AdjustmentTimeThreshold)
		{
			ServerLastClientAdjustmentTime = CurrentTime;

			if (ShouldUsePackedMovementRPCs())
			{
				ServerSendMoveResponse(ServerData->PendingAdjustment);
			}
			else
			{
				// Filling out a UPrimitiveComponent from the movement base struct for old functionality
				const bool bIsPlayingNetworkedRootMotionMontage = CharacterOwner->IsPlayingNetworkedRootMotionMontage();
				UPrimitiveComponent* MovementBase = Cast<UPrimitiveComponent>(ServerData->PendingAdjustment.NewMovementBasePhysicsObjectOwner.Get());
				const bool bHasMovementBase = MovementBase != nullptr;
				
				if (CurrentRootMotion.HasActiveRootMotionSources())
				{
					FRotator Rotation = ServerData->PendingAdjustment.NewRot.GetNormalized();
					FVector_NetQuantizeNormal CompressedRotation(Rotation.Pitch / 180.f, Rotation.Yaw / 180.f, Rotation.Roll / 180.f);
					ClientAdjustRootMotionSourcePosition
					(
						ServerData->PendingAdjustment.TimeStamp,
						CurrentRootMotion,
						bIsPlayingNetworkedRootMotionMontage,
						bIsPlayingNetworkedRootMotionMontage ? CharacterOwner->GetRootMotionAnimMontageInstance()->GetPosition() : -1.f,
						ServerData->PendingAdjustment.NewLoc,
						CompressedRotation,
						GetGravitySpaceZ(ServerData->PendingAdjustment.NewVel),
						MovementBase,
						ServerData->PendingAdjustment.NewBaseBoneName,
						bHasMovementBase,
						ServerData->PendingAdjustment.bBaseRelativePosition,
						ServerData->PendingAdjustment.MovementMode
					);
				}
				else if (bIsPlayingNetworkedRootMotionMontage)
				{
					FRotator Rotation = ServerData->PendingAdjustment.NewRot.GetNormalized();
					FVector_NetQuantizeNormal CompressedRotation(Rotation.Pitch / 180.f, Rotation.Yaw / 180.f, Rotation.Roll / 180.f);
					ClientAdjustRootMotionPosition
					(
						ServerData->PendingAdjustment.TimeStamp,
						CharacterOwner->GetRootMotionAnimMontageInstance()->GetPosition(),
						ServerData->PendingAdjustment.NewLoc,
						CompressedRotation,
						GetGravitySpaceZ(ServerData->PendingAdjustment.NewVel),
						MovementBase,
						ServerData->PendingAdjustment.NewBaseBoneName,
						bHasMovementBase,
						ServerData->PendingAdjustment.bBaseRelativePosition,
						ServerData->PendingAdjustment.MovementMode
					);
				}
				else if (ServerData->PendingAdjustment.NewVel.IsZero())
				{
					ClientVeryShortAdjustPosition
					(
						ServerData->PendingAdjustment.TimeStamp,
						ServerData->PendingAdjustment.NewLoc,
						MovementBase,
						ServerData->PendingAdjustment.NewBaseBoneName,
						bHasMovementBase,
						ServerData->PendingAdjustment.bBaseRelativePosition,
						ServerData->PendingAdjustment.MovementMode
					);
				}
				else
				{
					ClientAdjustPosition
					(
						ServerData->PendingAdjustment.TimeStamp,
						ServerData->PendingAdjustment.NewLoc,
						ServerData->PendingAdjustment.NewVel,
						MovementBase,
						ServerData->PendingAdjustment.NewBaseBoneName,
						bHasMovementBase,
						ServerData->PendingAdjustment.bBaseRelativePosition,
						ServerData->PendingAdjustment.MovementMode
					);
				}
			}
		}
	}

	ServerData->PendingAdjustment.TimeStamp = 0;
	ServerData->PendingAdjustment.bAckGoodMove = false;
	ServerData->bForceClientUpdate = false;
}


void UCharacterMovementComponent::ClientVeryShortAdjustPosition(float TimeStamp, FVector NewLoc, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	CharacterOwner->ClientVeryShortAdjustPosition(TimeStamp, NewLoc, NewBase, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}


void UCharacterMovementComponent::ClientVeryShortAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	FMovementBaseInterfaceData NewMovementBaseInterfaceData(NewBase);
	ClientVeryShortAdjustPosition_Implementation(TimeStamp, NewLoc, &NewMovementBaseInterfaceData, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

void UCharacterMovementComponent::ClientVeryShortAdjustPosition_Implementation
	(
	float TimeStamp,
	FVector NewLoc,
	FMovementBaseInterfaceData* NewMovementBaseInterfaceData,
	FName NewBaseBoneName,
	bool bHasBase,
	bool bBaseRelativePosition,
	uint8 ServerMovementMode
	)
{
	if (HasValidData())
	{
		UPrimitiveComponent* MovementBase = NewMovementBaseInterfaceData ? Cast<UPrimitiveComponent>(NewMovementBaseInterfaceData->PhysicsObjectOwner.Get()) : nullptr;
		ClientAdjustPosition(TimeStamp, NewLoc, FVector::ZeroVector, MovementBase, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
	}
}


void UCharacterMovementComponent::ClientAdjustPosition(float TimeStamp, FVector NewLoc, FVector NewVel, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	CharacterOwner->ClientAdjustPosition(TimeStamp, NewLoc, NewVel, NewBase, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}


void UCharacterMovementComponent::ClientAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, FVector NewVel, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode, TOptional<FRotator> OptionalRotation)
{
	FMovementBaseInterfaceData NewMovementBaseInterfaceData(NewBase);
	ClientAdjustPosition_Implementation(TimeStamp, NewLoc, NewVel, &NewMovementBaseInterfaceData, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode, OptionalRotation);
}

void UCharacterMovementComponent::ClientAdjustPosition_Implementation
	(
	float TimeStamp,
	FVector NewLocation,
	FVector NewVelocity,
	FMovementBaseInterfaceData* NewMovementBaseInterfaceData,
	FName NewBaseBoneName,
	bool bHasBase,
	bool bBaseRelativePosition,
	uint8 ServerMovementMode,
	TOptional<FRotator> OptionalRotation /* = TOptional<FRotator>()*/
	)
{
	if (!HasValidData() || !IsActive())
	{
		return;
	}

	// Skip applying server corrections when flagged. AckMove() still prunes SavedMoves to prevent unbounded growth.
	if (bClientIgnoreMovementCorrections)
	{
		FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
		if (ClientData)
		{
			const int32 MoveIndex = ClientData->GetSavedMoveIndex(TimeStamp);
			if (MoveIndex != INDEX_NONE)
			{
				ClientData->AckMove(MoveIndex, *this);
			}
		}

		UE_LOG(LogNetPlayerMovement, Verbose, TEXT("ClientAdjustPosition_Implementation: Ignoring server correction. bClientIgnoreMovementCorrections is set. TimeStamp: %f"), TimeStamp);

		return;
	}

	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	check(ClientData);
	
	// Make sure the base actor exists on this client.
	const bool bUnresolvedBase = bHasBase && !MovementBaseUtility::IsMovementBaseDataValid(NewMovementBaseInterfaceData);
	if (bUnresolvedBase)
	{
		if (bBaseRelativePosition)
		{
			UE_LOGF(LogNetPlayerMovement, Warning, "ClientAdjustPosition_Implementation could not resolve the new relative movement base actor, ignoring server correction! Client currently at world location %ls on base %ls",
				*UpdatedComponent->GetComponentLocation().ToString(), *GetNameSafe(NewMovementBaseInterfaceData ? NewMovementBaseInterfaceData->GetMovementBaseObject() : nullptr));
			return;
		}
		else
		{
			UE_LOGF(LogNetPlayerMovement, Verbose, "ClientAdjustPosition_Implementation could not resolve the new absolute movement base actor, but WILL use the position!");
		}
	}
	
	// Ack move if it has not expired.
	int32 MoveIndex = ClientData->GetSavedMoveIndex(TimeStamp);
	if( MoveIndex == INDEX_NONE )
	{
		if( ClientData->LastAckedMove.IsValid() )
		{
			UE_LOGF(LogNetPlayerMovement, Log,  "ClientAdjustPosition_Implementation could not find Move for TimeStamp: %f, LastAckedTimeStamp: %f, CurrentTimeStamp: %f", TimeStamp, ClientData->LastAckedMove->TimeStamp, ClientData->CurrentTimeStamp);
		}
		return;
	}

	ClientData->AckMove(MoveIndex, *this);
	
	FVector WorldShiftedNewLocation;
	//  Received Location is relative to dynamic base
	if (bBaseRelativePosition)
	{
		MovementBaseUtility::TransformLocationToWorld(NewMovementBaseInterfaceData, NewBaseBoneName, NewLocation, WorldShiftedNewLocation); // TODO: error handling if returns false	
	}
	else
	{
		WorldShiftedNewLocation = FRepMovement::RebaseOntoLocalOrigin(NewLocation, this);
	}

	// Server's world velocity may need to be converted to velocity relative to the movement base orientation, if the base orientations don't match.
	const FCharacterMoveResponseDataContainer& ResponseDataContainer = GetMoveResponseDataContainer();
	if (ResponseDataContainer.ClientAdjustment.bBaseRelativeVelocity)
	{
		// Convert Relative Velocity -> World Velocity
		const FVector CurrentVelocity = NewVelocity;
		MovementBaseUtility::TransformDirectionToWorld(NewMovementBaseInterfaceData, NewBaseBoneName, CurrentVelocity, NewVelocity);
	}

	// Fall back to the last-known good rotation if the server didn't send one
	if (CharacterMovementCVars::bUseLastGoodRotationDuringCorrection
		&& (bOrientRotationToMovement || bUseControllerDesiredRotation)
		&& (!OptionalRotation.IsSet() && ClientData->LastAckedMove.IsValid()))
	{
		OptionalRotation = ClientData->LastAckedMove->SavedRotation;
	}

	// Trigger event
	OnClientCorrectionReceived(*ClientData, TimeStamp, WorldShiftedNewLocation, NewVelocity, NewMovementBaseInterfaceData, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode, DefaultGravityDirection);

	// Trust the server's positioning.
	if (UpdatedComponent)
	{
		if (OptionalRotation.IsSet())
		{
			UpdatedComponent->SetWorldLocationAndRotation(WorldShiftedNewLocation, OptionalRotation.GetValue(), false, nullptr, ETeleportType::TeleportPhysics);
		}
		else
		{
			UpdatedComponent->SetWorldLocation(WorldShiftedNewLocation, false, nullptr, ETeleportType::TeleportPhysics);
		}
	}

	Velocity = NewVelocity;

	// Trust the server's movement mode
	ApplyNetworkMovementMode(ServerMovementMode);

	// Set base component
	FMovementBaseInterfaceData FinalMovementBaseInterfaceData = NewMovementBaseInterfaceData;
	FName FinalBaseBoneName = NewBaseBoneName;
	if (bUnresolvedBase)
	{
		check(!FinalMovementBaseInterfaceData.IsValid());
		check(!bBaseRelativePosition);
		
		// We had an unresolved base from the server
		// If walking, we'd like to continue walking if possible, to avoid falling for a frame, so try to find a base where we moved to.
		if (MovementBaseUtility::IsMovementBaseDataValid(GetMovementBaseInterfaceData()) && UpdatedComponent)
		{
			FindFloor(UpdatedComponent->GetComponentLocation(), CurrentFloor, false);
			if (CurrentFloor.IsWalkableFloor())
			{
				FinalMovementBaseInterfaceData = MovementBaseUtility::GetMovementBaseDataFromHitResult(&CurrentFloor.HitResult);
				FinalBaseBoneName = CurrentFloor.HitResult.BoneName;
			}
			else
			{
				FinalMovementBaseInterfaceData.Clear();
				FinalBaseBoneName = NAME_None;
			}
		}
	}
	SetBase(&FinalMovementBaseInterfaceData, FinalBaseBoneName);

	// Update floor at new location
	UpdateFloorFromAdjustment();
	bJustTeleported = true;

	// Even if base has not changed, we need to recompute the relative offsets (since we've moved).
	SaveBaseLocation();
	
	LastUpdateLocation = UpdatedComponent ? UpdatedComponent->GetComponentLocation() : FVector::ZeroVector;
	LastUpdateRotation = UpdatedComponent ? UpdatedComponent->GetComponentQuat() : FQuat::Identity;
	LastUpdateVelocity = Velocity;

	UpdateComponentVelocity();
	ClientData->bUpdatePosition = true;
}


void UCharacterMovementComponent::ClientAdjustRootMotionPosition(float TimeStamp, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	CharacterOwner->ClientAdjustRootMotionPosition(TimeStamp, ServerMontageTrackPosition, ServerLoc, ServerRotation, ServerVelZ, ServerBase, ServerBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

void UCharacterMovementComponent::OnClientCorrectionReceived(FNetworkPredictionData_Client_Character& ClientData, float TimeStamp, FVector NewLocation, FVector NewVelocity, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode, FVector ServerGravityDirection)
{
	FMovementBaseInterfaceData NewMovementBaseInterfaceData(NewBase);
	OnClientCorrectionReceived(ClientData, TimeStamp, NewLocation, NewVelocity, &NewMovementBaseInterfaceData, NewBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode, ServerGravityDirection);
}

void UCharacterMovementComponent::OnClientCorrectionReceived(FNetworkPredictionData_Client_Character& ClientData, float TimeStamp, FVector NewLocation, FVector NewVelocity, FMovementBaseInterfaceData* NewMovementBaseInterfaceData, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode, FVector ServerGravityDirection)
{
#if CSV_PROFILER_STATS
	const bool bIsRecordingDetailedCsvStats = FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(CharacterMovementDetailed));
	if (bIsRecordingDetailedCsvStats)
	{
		if (CharacterMovementCVars::bClientRecordNetCorrectionDistanceToCsv)
		{
			const FVector ClientLocAtCorrectedMove = ClientData.LastAckedMove.IsValid() ? ClientData.LastAckedMove->SavedLocation : UpdatedComponent->GetComponentLocation();
			const FVector LocDiff = ClientLocAtCorrectedMove - NewLocation;
			const double LocDiffSize = LocDiff.Size();
			CSV_CUSTOM_STAT(CharacterMovementDetailed, Correction_Distance, LocDiffSize, ECsvCustomStatOp::Set);
		}
	}
#endif //CSV_PROFILER_STATS

#if !UE_BUILD_SHIPPING
	if (CharacterMovementCVars::NetShowCorrections != 0)
	{
		const FVector ClientLocAtCorrectedMove = ClientData.LastAckedMove.IsValid() ? ClientData.LastAckedMove->SavedLocation : UpdatedComponent->GetComponentLocation();
		const FVector LocDiff = ClientLocAtCorrectedMove - NewLocation;
		const UObject* MovementBaseObject = NewMovementBaseInterfaceData ? NewMovementBaseInterfaceData->GetMovementBaseObject() : nullptr;
		const FString NewBaseString = MovementBaseUtility::IsMovementBaseDataValid(NewMovementBaseInterfaceData) ? MovementBaseObject->GetPathName(MovementBaseObject->GetOutermost()) : TEXT("None");
		UE_LOGF(LogNetPlayerMovement, Warning, "*** Client: Error for %ls at Time=%.3f is %3.3f LocDiff(%ls) ClientLoc(%ls) ServerLoc(%ls) NewBase: %ls NewBone: %ls ClientVel(%ls) ServerVel(%ls) SavedMoves %d",
			   *GetNameSafe(CharacterOwner), TimeStamp, LocDiff.Size(), *LocDiff.ToString(), *ClientLocAtCorrectedMove.ToString(), *NewLocation.ToString(), *NewBaseString, *NewBaseBoneName.ToString(), *Velocity.ToString(), *NewVelocity.ToString(), ClientData.SavedMoves.Num());
		const float DebugLifetime = CharacterMovementCVars::NetCorrectionLifetime;
		if (!LocDiff.IsNearlyZero())
		{
			// When server corrects us to a new location, draw red at location where client thought they were, green where the server corrected us to
			DrawDebugCapsule(GetWorld(), ClientLocAtCorrectedMove, CharacterOwner->GetSimpleCollisionHalfHeight(), CharacterOwner->GetSimpleCollisionRadius(), FQuat::Identity, FColor(255, 100, 100), false, DebugLifetime);
			DrawDebugCapsule(GetWorld(), NewLocation, CharacterOwner->GetSimpleCollisionHalfHeight(), CharacterOwner->GetSimpleCollisionRadius(), FQuat::Identity, FColor(100, 255, 100), false, DebugLifetime);
		}
		else
		{
			// When we receive a server correction that doesn't change our position from where our client move had us, draw yellow (otherwise would be overlapping)
			// This occurs when we receive an initial correction, replay moves to get us into the right location, and then receive subsequent corrections by the server (who doesn't know if we corrected already
			// so continues to send corrections). This is a "no-op" server correction with regards to location since we already corrected (occurs with latency)
			DrawDebugCapsule(GetWorld(), NewLocation, CharacterOwner->GetSimpleCollisionHalfHeight(), CharacterOwner->GetSimpleCollisionRadius(), FQuat::Identity, FColor(255, 255, 100), false, DebugLifetime);
		}
	}
#endif //!UE_BUILD_SHIPPING

#if ROOT_MOTION_DEBUG
	if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
	{
		const FVector VelocityCorrection = NewVelocity - Velocity;
		FString AdjustedDebugString = FString::Printf(TEXT("PerformMovement ClientAdjustPosition_Implementation Velocity(%s) OldVelocity(%s) Correction(%s) TimeStamp(%f)"),
													  *NewVelocity.ToCompactString(), *Velocity.ToCompactString(), *VelocityCorrection.ToCompactString(), TimeStamp);
		RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
	}
#endif //ROOT_MOTION_DEBUG
}

void UCharacterMovementComponent::ClientAdjustRootMotionPosition_Implementation(float TimeStamp, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	FMovementBaseInterfaceData ServerMovementBaseInterfaceData(ServerBase);
	ClientAdjustRootMotionPosition_Implementation(TimeStamp, ServerMontageTrackPosition, ServerLoc, ServerRotation, ServerVelZ, &ServerMovementBaseInterfaceData, ServerBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

void UCharacterMovementComponent::ClientAdjustRootMotionPosition_Implementation(
	float TimeStamp,
	float ServerMontageTrackPosition,
	FVector ServerLoc,
	FVector_NetQuantizeNormal ServerRotation,
	float ServerVelZ,
	FMovementBaseInterfaceData* ServerMovementBaseInterfaceData,
	FName ServerBaseBoneName,
	bool bHasBase,
	bool bBaseRelativePosition,
	uint8 ServerMovementMode)
{
	if (!HasValidData() || !IsActive())
	{
		return;
	}

	// Call ClientAdjustPosition first. This will Ack the move if it's not outdated.
	ClientAdjustPosition_Implementation(TimeStamp, ServerLoc, ServerVelZ * -GetGravityDirection(), ServerMovementBaseInterfaceData, ServerBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
	
	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	check(ClientData);

	// If this adjustment wasn't acknowledged (because outdated), then abort.
	if( !ClientData->LastAckedMove.IsValid() || (ClientData->LastAckedMove->TimeStamp != TimeStamp) )
	{
		return;
	}

	// We're going to replay Root Motion. This is relative to the Pawn's rotation, so we need to reset that as well.
	FRotator DecompressedRot(ServerRotation.X * 180.f, ServerRotation.Y * 180.f, ServerRotation.Z * 180.f);
	CharacterOwner->SetActorRotation(DecompressedRot);
	const FVector ServerLocation(FRepMovement::RebaseOntoLocalOrigin(ServerLoc, UpdatedComponent));
	UE_LOGF(LogRootMotion, Log,  "ClientAdjustRootMotionPosition_Implementation TimeStamp: %f, ServerMontageTrackPosition: %f, ServerLocation: %ls, ServerRotation: %ls, ServerVelZ: %f, ServerBase: %ls",
		TimeStamp, ServerMontageTrackPosition, *ServerLocation.ToCompactString(), *DecompressedRot.ToCompactString(), ServerVelZ, *GetNameSafe(ServerMovementBaseInterfaceData ? ServerMovementBaseInterfaceData->GetMovementBaseObject() : nullptr));

#if 0
	// DEBUG - get some insight on where errors came from
	if( false )
	{
		const FVector DeltaLocation = ServerLocation - ClientData->LastAckedMove->SavedLocation;
		const FRotator DeltaRotation = (DecompressedRot - ClientData->LastAckedMove->SavedRotation).GetNormalized();
		const float DeltaTrackPosition = (ServerMontageTrackPosition - ClientData->LastAckedMove->RootMotionTrackPosition);
		const float DeltaVelZ = (ServerVelZ - ClientData->LastAckedMove->SavedVelocity.Z);

		UE_LOGF(LogRootMotion, Log,  "\tErrors DeltaLocation: %ls, DeltaRotation: %ls, DeltaTrackPosition: %f",
			*DeltaLocation.ToCompactString(), *DeltaRotation.ToCompactString(), DeltaTrackPosition );
	}
#endif

	// Server disagrees with Client on the Root Motion AnimMontage Track position.
	if( CharacterOwner->bClientResimulateRootMotion || (ServerMontageTrackPosition != ClientData->LastAckedMove->RootMotionTrackPosition) )
	{
		// Not much we can do there unfortunately, just jump to server's track position.
		FAnimMontageInstance * RootMotionMontageInstance = CharacterOwner->GetRootMotionAnimMontageInstance();
		if (RootMotionMontageInstance && !RootMotionMontageInstance->IsRootMotionDisabled())
		{
			UE_LOGF(LogRootMotion, Log, "\tServer disagrees with Client's track position!! ServerTrackPosition: %f, ClientTrackPosition: %f, DeltaTrackPosition: %f. TimeStamp: %f, Character: %ls, Montage: %ls",
					ServerMontageTrackPosition, ClientData->LastAckedMove->RootMotionTrackPosition, (ServerMontageTrackPosition - ClientData->LastAckedMove->RootMotionTrackPosition), TimeStamp, *GetNameSafe(CharacterOwner), *GetNameSafe(RootMotionMontageInstance->Montage));
	
			RootMotionMontageInstance->SetPosition(ServerMontageTrackPosition);
			CharacterOwner->bClientResimulateRootMotion = true;
		}
	}
}


void UCharacterMovementComponent::ClientAdjustRootMotionSourcePosition(float TimeStamp, FRootMotionSourceGroup ServerRootMotion, bool bHasAnimRootMotion, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	CharacterOwner->ClientAdjustRootMotionSourcePosition(TimeStamp, ServerRootMotion, bHasAnimRootMotion, ServerMontageTrackPosition, ServerLoc, ServerRotation, ServerVelZ, ServerBase, ServerBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

void UCharacterMovementComponent::ClientAdjustRootMotionSourcePosition_Implementation(float TimeStamp, FRootMotionSourceGroup ServerRootMotion, bool bHasAnimRootMotion, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode)
{
	FMovementBaseInterfaceData ServerMovementBaseInterfaceData(ServerBase);
	ClientAdjustRootMotionSourcePosition_Implementation(TimeStamp, ServerRootMotion, bHasAnimRootMotion, ServerMontageTrackPosition, ServerLoc, ServerRotation, ServerVelZ, &ServerMovementBaseInterfaceData, ServerBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
}

void UCharacterMovementComponent::ClientAdjustRootMotionSourcePosition_Implementation(
	float TimeStamp,
	FRootMotionSourceGroup ServerRootMotion,
	bool bHasAnimRootMotion,
	float ServerMontageTrackPosition,
	FVector ServerLoc,
	FVector_NetQuantizeNormal ServerRotation,
	float ServerVelZ,
	FMovementBaseInterfaceData* ServerMovementBaseInterfaceData,
	FName ServerBaseBoneName,
	bool bHasBase,
	bool bBaseRelativePosition,
	uint8 ServerMovementMode)
{
	if (!HasValidData() || !IsActive())
	{
		return;
	}

#if ROOT_MOTION_DEBUG
	if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
	{
		FString AdjustedDebugString = FString::Printf(TEXT("ClientAdjustRootMotionSourcePosition_Implementation TimeStamp(%f)"),
			TimeStamp);
		RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
	}
#endif

	// Call ClientAdjustPosition first. This will Ack the move if it's not outdated.
	ClientAdjustPosition_Implementation(TimeStamp, ServerLoc, ServerVelZ * -GetGravityDirection(), ServerMovementBaseInterfaceData, ServerBaseBoneName, bHasBase, bBaseRelativePosition, ServerMovementMode);
	
	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	check(ClientData);

	// If this adjustment wasn't acknowledged (because outdated), then abort.
	if( !ClientData->LastAckedMove.IsValid() || (ClientData->LastAckedMove->TimeStamp != TimeStamp) )
	{
		return;
	}

	// We're going to replay Root Motion. This can be relative to the Pawn's rotation, so we need to reset that as well.
	FRotator DecompressedRot(ServerRotation.X * 180.f, ServerRotation.Y * 180.f, ServerRotation.Z * 180.f);
	CharacterOwner->SetActorRotation(DecompressedRot);
	const FVector ServerLocation(FRepMovement::RebaseOntoLocalOrigin(ServerLoc, UpdatedComponent));
	UE_LOGF(LogRootMotion, Log,  "ClientAdjustRootMotionSourcePosition_Implementation TimeStamp: %f, NumRootMotionSources: %d, ServerLocation: %ls, ServerRotation: %ls, ServerVelZ: %f, ServerBase: %ls",
		TimeStamp, ServerRootMotion.RootMotionSources.Num(), *ServerLocation.ToCompactString(), *DecompressedRot.ToCompactString(), ServerVelZ, *GetNameSafe(ServerMovementBaseInterfaceData ? ServerMovementBaseInterfaceData->GetMovementBaseObject() : nullptr));

	// Handle AnimRootMotion correction
	if (bHasAnimRootMotion)
	{
		// DEBUG - get some insight on where errors came from
		#if 0
		{
			const FVector DeltaLocation = ServerLocation - ClientData->LastAckedMove->SavedLocation;
			const FRotator DeltaRotation = (DecompressedRot - ClientData->LastAckedMove->SavedRotation).GetNormalized();
			const float DeltaTrackPosition = (ServerMontageTrackPosition - ClientData->LastAckedMove->RootMotionTrackPosition);
			const float DeltaVelZ = (ServerVelZ - ClientData->LastAckedMove->SavedVelocity.Z);

			UE_LOGF(LogRootMotion, Log,  "\tErrors DeltaLocation: %ls, DeltaRotation: %ls, DeltaTrackPosition: %f",
				*DeltaLocation.ToCompactString(), *DeltaRotation.ToCompactString(), DeltaTrackPosition );
		}
		#endif

		// Server disagrees with Client on the Root Motion AnimMontage Track position.
		if( CharacterOwner->bClientResimulateRootMotion || (ServerMontageTrackPosition != ClientData->LastAckedMove->RootMotionTrackPosition) )
		{
			UE_LOGF(LogRootMotion, Log,  "\tServer disagrees with Client's track position!! ServerTrackPosition: %f, ClientTrackPosition: %f, DeltaTrackPosition: %f. TimeStamp: %f",
				ServerMontageTrackPosition, ClientData->LastAckedMove->RootMotionTrackPosition, (ServerMontageTrackPosition - ClientData->LastAckedMove->RootMotionTrackPosition), TimeStamp);

			// Not much we can do there unfortunately, just jump to server's track position.
			FAnimMontageInstance * RootMotionMontageInstance = CharacterOwner->GetRootMotionAnimMontageInstance();
			if (RootMotionMontageInstance && !RootMotionMontageInstance->IsRootMotionDisabled())
			{
				RootMotionMontageInstance->SetPosition(ServerMontageTrackPosition);
				CharacterOwner->bClientResimulateRootMotion = true;
			}
		}
	}

	// First we need to convert Server IDs -> Local IDs in ServerRootMotion for comparison
	ConvertRootMotionServerIDsToLocalIDs(ClientData->LastAckedMove->SavedRootMotion, ServerRootMotion, TimeStamp);

	// Cull ServerRootMotion of any root motion sources that don't match ones we have in this move
	ServerRootMotion.CullInvalidSources();

	// Server disagrees with Client on Root Motion state.
	if( CharacterOwner->bClientResimulateRootMotionSources || (ServerRootMotion != ClientData->LastAckedMove->SavedRootMotion) )
	{
		if (!CharacterOwner->bClientResimulateRootMotionSources)
		{
			UE_LOGF(LogNetPlayerMovement, VeryVerbose, "ClientAdjustRootMotionSourcePosition called, server/LastAckedMove mismatch");
		}

		CharacterOwner->SavedRootMotion = ServerRootMotion;
		CharacterOwner->bClientResimulateRootMotionSources = true;
	}
}

void UCharacterMovementComponent::ClientAckGoodMove(float TimeStamp)
{
	CharacterOwner->ClientAckGoodMove(TimeStamp);
}

void UCharacterMovementComponent::ClientAckGoodMove_Implementation(float TimeStamp)
{
	if (!HasValidData() || !IsActive())
	{
		return;
	}

	FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
	check(ClientData);

#if ROOT_MOTION_DEBUG
	if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() == 1)
	{
		FString AdjustedDebugString = FString::Printf(TEXT("ClientAckGoodMove_Implementation TimeStamp(%f)"),
			TimeStamp);
		RootMotionSourceDebug::PrintOnScreen(*CharacterOwner, AdjustedDebugString);
	}
#endif

	// Ack move if it has not expired.
	int32 MoveIndex = ClientData->GetSavedMoveIndex(TimeStamp);
	if( MoveIndex == INDEX_NONE )
	{
		if( ClientData->LastAckedMove.IsValid() )
		{
			UE_LOGF(LogNetPlayerMovement, Log, "ClientAckGoodMove_Implementation could not find Move for TimeStamp: %f, LastAckedTimeStamp: %f, CurrentTimeStamp: %f", TimeStamp, ClientData->LastAckedMove->TimeStamp, ClientData->CurrentTimeStamp);
		}
		return;
	}

	ClientData->AckMove(MoveIndex, *this);
}


void UCharacterMovementComponent::CapsuleTouched(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult )
{
	if (!bEnablePhysicsInteraction)
	{
		return;
	}

	if (OtherComp != NULL && OtherComp->IsAnySimulatingPhysics())
	{
		const FVector OtherLoc = OtherComp->GetComponentLocation();
		const FVector Loc = UpdatedComponent->GetComponentLocation();
		
		FVector ImpulseDir = OtherLoc - Loc;
		SetGravitySpaceZ(ImpulseDir, 0.25f);
		ImpulseDir = (ImpulseDir.GetSafeNormal() + ProjectToGravityFloor(Velocity).GetSafeNormal()) * 0.5f;
		ImpulseDir.Normalize();

		FName BoneName = NAME_None;
		if (OtherBodyIndex != INDEX_NONE)
		{
			BoneName = ((USkinnedMeshComponent*)OtherComp)->GetBoneName(OtherBodyIndex);
		}

		float TouchForceFactorModified = TouchForceFactor;

		if ( bTouchForceScaledToMass )
		{
			FBodyInstance* BI = OtherComp->GetBodyInstance(BoneName);
			TouchForceFactorModified *= BI ? BI->GetBodyMass() : 1.0f;
		}

		float ImpulseStrength = FMath::Clamp<FVector::FReal>(ProjectToGravityFloor(Velocity).Size() * TouchForceFactorModified, 
			MinTouchForce > 0.0f ? MinTouchForce : -FLT_MAX, 
			MaxTouchForce > 0.0f ? MaxTouchForce : FLT_MAX);

		FVector Impulse = ImpulseDir * ImpulseStrength;

		OtherComp->AddImpulse(Impulse, BoneName);
	}
}

FVector UCharacterMovementComponent::GetLastUpdateRequestedVelocity() const
{
	return LastUpdateRequestedVelocity;
}

void UCharacterMovementComponent::SetAvoidanceGroup(int32 GroupFlags)
{
	SetAvoidanceGroupMask(GroupFlags);
}

void UCharacterMovementComponent::SetAvoidanceGroupMask(const FNavAvoidanceMask& GroupMask)
{
	SetAvoidanceGroupMask(GroupMask.Packed);
}

void UCharacterMovementComponent::SetGroupsToAvoid(int32 GroupFlags)
{
	SetGroupsToAvoidMask(GroupFlags);
}

void UCharacterMovementComponent::SetGroupsToAvoidMask(const FNavAvoidanceMask& GroupMask)
{
	SetGroupsToAvoidMask(GroupMask.Packed);
}

void UCharacterMovementComponent::SetGroupsToIgnore(int32 GroupFlags)
{
	SetGroupsToIgnoreMask(GroupFlags);
}

void UCharacterMovementComponent::SetGroupsToIgnoreMask(const FNavAvoidanceMask& GroupMask)
{
	SetGroupsToIgnoreMask(GroupMask.Packed);
}

void UCharacterMovementComponent::SetAvoidanceEnabled(bool bEnable)
{
	if (bUseRVOAvoidance != bEnable)
	{
		bUseRVOAvoidance = bEnable;

		const int32 OldAvoidanceUID = AvoidanceUID;
		AvoidanceUID = 0;

		// this is a safety check - it's possible to not have CharacterOwner at this point if this function gets
		// called too early
		ensure(GetCharacterOwner());
		if (GetCharacterOwner() != nullptr)
		{
			UWorld* World = GetWorld();
			UAvoidanceManager* AvoidanceManager = World ? World->GetAvoidanceManager() : nullptr;

			if (AvoidanceManager)
			{
				if (bEnable)
				{
					AvoidanceManager->RegisterMovementComponent(this, AvoidanceWeight);
				}
				else if (!AvoidanceManager->IsAutoPurgeEnabled())
				{
					// When disabling avoidance the object needs to be removed immediately if the manager isn't set to auto purge. 
					// Otherwise we would leak the object since there isn't anything left to remove it.
					AvoidanceManager->RemoveAvoidanceObject(OldAvoidanceUID);
				}
			}
		}
	}
}

void UCharacterMovementComponent::ApplyDownwardForce(float DeltaSeconds)
{
	if (StandingDownwardForceScale != 0.0f && CurrentFloor.HitResult.IsValidBlockingHit())
	{
		UPrimitiveComponent* BaseComp = CurrentFloor.HitResult.GetComponent();
		const FVector Gravity = -GetGravityDirection() * GetGravityZ();

		if (BaseComp && BaseComp->IsAnySimulatingPhysics() && !Gravity.IsZero())
		{
			BaseComp->AddForceAtLocation(Gravity * Mass * StandingDownwardForceScale, CurrentFloor.HitResult.ImpactPoint, CurrentFloor.HitResult.BoneName);
		}
	}
}

void UCharacterMovementComponent::ApplyRepulsionForce(float DeltaSeconds)
{
	if (UpdatedPrimitive && RepulsionForce > 0.0f && CharacterOwner!=nullptr)
	{
		const TArray<FOverlapInfo>& Overlaps = UpdatedPrimitive->GetOverlapInfos();
		if (Overlaps.Num() > 0)
		{
			FCollisionQueryParams QueryParams (SCENE_QUERY_STAT(CMC_ApplyRepulsionForce));
			QueryParams.bReturnFaceIndex = false;
			QueryParams.bReturnPhysicalMaterial = false;

			float CapsuleRadius = 0.f;
			float CapsuleHalfHeight = 0.f;
			CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(CapsuleRadius, CapsuleHalfHeight);
			const float RepulsionForceRadius = CapsuleRadius * 1.2f;
			const float StopBodyDistance = 2.5f;
			const FVector MyLocation = UpdatedPrimitive->GetComponentLocation();

			for (int32 i=0; i < Overlaps.Num(); i++)
			{
				const FOverlapInfo& Overlap = Overlaps[i];

				UPrimitiveComponent* OverlapComp = Overlap.OverlapInfo.Component.Get();
				if (!OverlapComp || OverlapComp->Mobility < EComponentMobility::Movable)
				{ 
					continue; 
				}

				// Use the body instead of the component for cases where we have multi-body overlaps enabled
				FBodyInstance* OverlapBody = nullptr;
				const int32 OverlapBodyIndex = Overlap.GetBodyIndex();
				const USkeletalMeshComponent* SkelMeshForBody = (OverlapBodyIndex != INDEX_NONE) ? Cast<USkeletalMeshComponent>(OverlapComp) : nullptr;
				if (SkelMeshForBody != nullptr)
				{
					OverlapBody = SkelMeshForBody->Bodies.IsValidIndex(OverlapBodyIndex) ? SkelMeshForBody->Bodies[OverlapBodyIndex] : nullptr;
				}
				else
				{
					OverlapBody = OverlapComp->GetBodyInstance();
				}

				if (!OverlapBody)
				{
					UE_LOGF(LogCharacterMovement, Warning, "%ls could not find overlap body for body index %d", *GetName(), OverlapBodyIndex);
					continue;
				}

				if (!OverlapBody->IsInstanceSimulatingPhysics())
				{
					continue;
				}

				FTransform BodyTransform = OverlapBody->GetUnrealWorldTransform();

				FVector BodyVelocity = OverlapBody->GetUnrealWorldVelocity();
				FVector BodyLocation = BodyTransform.GetLocation();

				// Trace to get the hit location on the capsule
				FHitResult Hit;
				bool bHasHit = UpdatedPrimitive->LineTraceComponent(Hit, BodyLocation,
																	ProjectToGravityFloor(MyLocation) + GetGravitySpaceComponentZ(BodyLocation),
																	QueryParams);

				FVector HitLoc = Hit.ImpactPoint;
				bool bIsPenetrating = Hit.bStartPenetrating || Hit.PenetrationDepth > StopBodyDistance;

				// If we didn't hit the capsule, we're inside the capsule
				if (!bHasHit) 
				{
					HitLoc = BodyLocation;
					bIsPenetrating = true;
				}

				const float DistanceNow = ProjectToGravityFloor(HitLoc - BodyLocation).SizeSquared();
				const float DistanceLater = ProjectToGravityFloor(HitLoc - (BodyLocation + BodyVelocity * DeltaSeconds)).SizeSquared();

				if (bHasHit && DistanceNow < StopBodyDistance && !bIsPenetrating)
				{
					OverlapBody->SetLinearVelocity(FVector::ZeroVector, false);
				}
				else if (DistanceLater <= DistanceNow || bIsPenetrating)
				{
					FVector ForceCenter = MyLocation;

					if (bHasHit)
					{
						SetGravitySpaceZ(ForceCenter, GetGravitySpaceZ(HitLoc));
					}
					else
					{
						const FVector::FReal MyLocationZ = GetGravitySpaceZ(MyLocation);
						SetGravitySpaceZ(ForceCenter, FMath::Clamp(GetGravitySpaceZ(BodyLocation), MyLocationZ - CapsuleHalfHeight, MyLocationZ + CapsuleHalfHeight));
					}

					OverlapBody->AddRadialForceToBody(ForceCenter, RepulsionForceRadius, RepulsionForce * Mass, ERadialImpulseFalloff::RIF_Constant);
				}
			}
		}
	}
}

void UCharacterMovementComponent::ApplyAccumulatedForces(float DeltaSeconds)
{
	const FVector::FReal ImpulseToApplyZ = GetGravitySpaceZ(PendingImpulseToApply);
	const FVector::FReal ForceToApplyZ = GetGravitySpaceZ(PendingForceToApply);
	if (ImpulseToApplyZ != 0.0 || ForceToApplyZ != 0.0)
	{
		// check to see if applied momentum is enough to overcome gravity
		if (IsMovingOnGround() && (ImpulseToApplyZ + (ForceToApplyZ * DeltaSeconds) + (GetGravityZ() * DeltaSeconds) > UE_SMALL_NUMBER))
		{
			SetMovementMode(MOVE_Falling);
		}
	}

	Velocity += PendingImpulseToApply + (PendingForceToApply * DeltaSeconds);
	
	// Don't call ClearAccumulatedForces() because it could affect launch velocity
	PendingImpulseToApply = FVector::ZeroVector;
	PendingForceToApply = FVector::ZeroVector;
}

void UCharacterMovementComponent::ClearAccumulatedForces()
{
	PendingImpulseToApply = FVector::ZeroVector;
	PendingForceToApply = FVector::ZeroVector;
	PendingLaunchVelocity = FVector::ZeroVector;
}

void UCharacterMovementComponent::AddRadialForce(const FVector& Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff)
{
	FVector Delta = UpdatedComponent->GetComponentLocation() - Origin;
	const float DeltaMagnitude = Delta.Size();

	// Do nothing if outside radius
	if(DeltaMagnitude > Radius)
	{
		return;
	}

	Delta = Delta.GetSafeNormal();

	float ForceMagnitude = Strength;
	if (Falloff == RIF_Linear && Radius > 0.0f)
	{
		ForceMagnitude *= (1.0f - (DeltaMagnitude / Radius));
	}

	AddForce(Delta * ForceMagnitude);
}
 
void UCharacterMovementComponent::AddRadialImpulse(const FVector& Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bVelChange)
{
	FVector Delta = UpdatedComponent->GetComponentLocation() - Origin;
	const float DeltaMagnitude = Delta.Size();

	// Do nothing if outside radius
	if(DeltaMagnitude > Radius)
	{
		return;
	}

	Delta = Delta.GetSafeNormal();

	float ImpulseMagnitude = Strength;
	if (Falloff == RIF_Linear && Radius > 0.0f)
	{
		ImpulseMagnitude *= (1.0f - (DeltaMagnitude / Radius));
	}

	AddImpulse(Delta * ImpulseMagnitude, bVelChange);
}

void UCharacterMovementComponent::RegisterComponentTickFunctions(bool bRegister)
{
	Super::RegisterComponentTickFunctions(bRegister);

	if (bRegister)
	{
		if (CharacterMovementCVars::AsyncCharacterMovement == 1 && SetupActorComponentTickFunction(&PrePhysicsTickFunction))
		{
			PrePhysicsTickFunction.Target = this;
			PrePhysicsTickFunction.AddPrerequisite(this, this->PrimaryComponentTick);
		}
		if (SetupActorComponentTickFunction(&PostPhysicsTickFunction))
		{
			PostPhysicsTickFunction.Target = this;
			PostPhysicsTickFunction.AddPrerequisite(this, this->PrimaryComponentTick);
		}
	}
	else
	{
		if(PostPhysicsTickFunction.IsTickFunctionRegistered())
		{
			PostPhysicsTickFunction.UnRegisterTickFunction();
		}
		if (PrePhysicsTickFunction.IsTickFunctionRegistered())
		{
			PrePhysicsTickFunction.UnRegisterTickFunction();
		}
	}
}

void UCharacterMovementComponent::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	OldBaseLocation += InOffset;
	LastUpdateLocation += InOffset;

	if (CharacterOwner != nullptr && CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy)
	{
		FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
		if (ClientData != nullptr)
		{
			const int32 NumSavedMoves = ClientData->SavedMoves.Num();
			for (int32 i = 0; i < NumSavedMoves - 1; i++)
			{
				FSavedMove_Character* const CurrentMove = ClientData->SavedMoves[i].Get();
				CurrentMove->StartLocation += InOffset;
				CurrentMove->SavedLocation += InOffset;
			}

			if (FSavedMove_Character* const PendingMove = ClientData->PendingMove.Get())
			{
				PendingMove->StartLocation += InOffset;
				PendingMove->SavedLocation += InOffset;
			}
		}
	}
}

void UCharacterMovementComponent::TickCharacterPose(float DeltaTime)
{
	if (DeltaTime < UCharacterMovementComponent::MIN_TICK_TIME)
	{
		return;
	}

	check(CharacterOwner && CharacterOwner->GetMesh());
	USkeletalMeshComponent* CharacterMesh = CharacterOwner->GetMesh();

	// bAutonomousTickPose is set, we control TickPose from the Character's Movement and Networking updates, and bypass the Component's update.
	// (Or Simulating Root Motion for remote clients)
	CharacterMesh->bIsAutonomousTickPose = true;

	if (CharacterMesh->ShouldTickPose())
	{
		// Keep track of if we're playing root motion, just in case the root motion montage ends this frame.
		// Also cache the root motion translation scale, in case the root motion ends in TickPose and
		// translation scale is reset by a blend out listener.
		const bool bWasPlayingRootMotion = CharacterOwner->IsPlayingRootMotion();
		const float RootMotionTranslationScale = CharacterOwner->GetAnimRootMotionTranslationScale();

		if (CharacterMesh->bEnableAnimation)
		{
			CharacterMesh->TickPose(DeltaTime, true);
		}
		else
		{
			CharacterMesh->TickMontagesOnly(DeltaTime, true);
		}

		// Grab root motion now that we have ticked the pose
		if (CharacterOwner->IsPlayingRootMotion() || bWasPlayingRootMotion)
		{
			FRootMotionMovementParams RootMotion = CharacterMesh->ConsumeRootMotion();
			if (RootMotion.bHasRootMotion)
			{
				RootMotion.ScaleRootMotionTranslation(RootMotionTranslationScale);
				RootMotionParams.Accumulate(RootMotion);
			}

#if !(UE_BUILD_SHIPPING)
			// Debugging
			{
				FAnimMontageInstance* RootMotionMontageInstance = CharacterOwner->GetRootMotionAnimMontageInstance();
				UE_LOGF(LogRootMotion, Log, "UCharacterMovementComponent::TickCharacterPose Role: %ls, RootMotionMontage: %ls, MontagePos: %f, DeltaTime: %f, ExtractedRootMotion: %ls, AccumulatedRootMotion: %ls"
					, *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), CharacterOwner->GetLocalRole())
					, *GetNameSafe(RootMotionMontageInstance ? RootMotionMontageInstance->Montage : NULL)
					, RootMotionMontageInstance ? RootMotionMontageInstance->GetPosition() : -1.f
					, DeltaTime
					, *RootMotion.GetRootMotionTransform().GetTranslation().ToCompactString()
					, *RootMotionParams.GetRootMotionTransform().GetTranslation().ToCompactString()
					);
			}
#endif // !(UE_BUILD_SHIPPING)
		}

		// We need to dispatch any queued events, since these can affect the next move request, after we ticked the character pose.
		if (CharacterMovementCVars::EnableQueuedAnimEventsOnServer)
		{
			UAnimInstance* AnimInstance = CharacterMesh->GetAnimInstance();
			UAnimInstance* PostProcessAnimInstance = CharacterMesh->GetPostProcessInstance();
			
			if (AnimInstance || PostProcessAnimInstance)
			{
				const bool bIsAnimGraphUpdateEnabled = CharacterMesh->VisibilityBasedAnimTickOption <= EVisibilityBasedAnimTickOption::AlwaysTickPose;
				const bool bIsMainAnimGraphUpdatePending = AnimInstance ? AnimInstance->NeedsUpdate() : false;
				const bool bIsPostProcessAnimGraphUpdatePending =  PostProcessAnimInstance ? PostProcessAnimInstance->NeedsUpdate() : false;
				const bool bIsAnimGraphUpdatePending = bIsMainAnimGraphUpdatePending || bIsPostProcessAnimGraphUpdatePending;
				
				if (bIsAnimGraphUpdateEnabled && bIsAnimGraphUpdatePending)
				{
					// If we are doing a full graph update on the server but its doing a parallel update,
					// trigger events right away since these are notifies queued from the montage update, and we could be receiving multiple ServerMoves per frame.
					CharacterMesh->ConditionallyDispatchQueuedAnimEvents();

					// We need to manually clear the anim notify queue (since normally its only is cleared in PreUpdateAnimation()) otherwise if animation ticks, the notifies queued from the ServerMove would fire twice.
					CharacterMesh->ForEachAnimInstance([](UAnimInstance* InAnimInstance)
					{
						check(InAnimInstance);
						InAnimInstance->ClearQueuedAnimEvents(false);
					});

					// When animation ticks, we want its queued events to be triggered.
					CharacterMesh->AllowQueuedAnimEventsNextDispatch();
				}
			}
		}
	}

	CharacterMesh->bIsAutonomousTickPose = false;
}

/** 
*	Root Motion
*/

bool UCharacterMovementComponent::HasRootMotionSources() const
{
	return CurrentRootMotion.HasActiveRootMotionSources() || (CharacterOwner && CharacterOwner->IsPlayingRootMotion() && CharacterOwner->GetMesh());
}

uint16 UCharacterMovementComponent::ApplyRootMotionSource(TSharedPtr<FRootMotionSource> SourcePtr)
{
	if (ensure(SourcePtr.IsValid()))
	{
		// Set default StartTime if it hasn't been set manually
		if (!SourcePtr->IsStartTimeValid())
		{
			if (CharacterOwner)
			{
				if (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy)
				{
					// Autonomous defaults to local timestamp
					FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
					if (ClientData)
					{
						SourcePtr->StartTime = ClientData->CurrentTimeStamp;
					}
				}
				else if (CharacterOwner->GetLocalRole() == ROLE_Authority && !IsNetMode(NM_Client))
				{
					// Authority defaults to current client time stamp, meaning it'll start next tick if not corrected
					FNetworkPredictionData_Server_Character* ServerData = GetPredictionData_Server_Character();
					if (ServerData)
					{
						SourcePtr->StartTime = ServerData->CurrentClientTimeStamp;
					}
				}
			}
		}

		OnRootMotionSourceBeingApplied(SourcePtr.Get());

		return CurrentRootMotion.ApplyRootMotionSource(SourcePtr);
	}

	return (uint16)ERootMotionSourceID::Invalid;
}

void UCharacterMovementComponent::OnRootMotionSourceBeingApplied(const FRootMotionSource* Source)
{
}

TSharedPtr<FRootMotionSource> UCharacterMovementComponent::GetRootMotionSource(FName InstanceName)
{
	return CurrentRootMotion.GetRootMotionSource(InstanceName);
}

TSharedPtr<FRootMotionSource> UCharacterMovementComponent::GetRootMotionSourceByID(uint16 RootMotionSourceID)
{
	return CurrentRootMotion.GetRootMotionSourceByID(RootMotionSourceID);
}

void UCharacterMovementComponent::RemoveRootMotionSource(FName InstanceName)
{
	CurrentRootMotion.RemoveRootMotionSource(InstanceName);
}

void UCharacterMovementComponent::RemoveRootMotionSourceByID(uint16 RootMotionSourceID)
{
	CurrentRootMotion.RemoveRootMotionSourceByID(RootMotionSourceID);
}

void UCharacterMovementComponent::ConvertRootMotionServerIDsToLocalIDs(const FRootMotionSourceGroup& LocalRootMotionToMatchWith, FRootMotionSourceGroup& InOutServerRootMotion, float TimeStamp)
{
	// Remove out of date mappings, they can never be used again.
	for (int32 MappingIndex = 0; MappingIndex < RootMotionIDMappings.Num(); MappingIndex++)
	{
		if (RootMotionIDMappings[MappingIndex].IsStillValid(TimeStamp))
		{
			// MappingIndex is valid, remove anything before it.
			const int32 CutOffIndex = MappingIndex - 1;
			if (CutOffIndex >= 0)
			{
				// Most recent entries added last, so we can cull the top of the list.
				RootMotionIDMappings.RemoveAt(0, CutOffIndex + 1, EAllowShrinking::No);
				break;
			}
		}
	}

	// Remove mappings that don't map to an active local root motion source.
	for (int32 MappingIndex = RootMotionIDMappings.Num()-1; MappingIndex>=0; MappingIndex--)
	{
		bool bFoundLocalSource = false;
		for (const TSharedPtr<FRootMotionSource>& LocalRootMotionSource : LocalRootMotionToMatchWith.RootMotionSources)
		{
			if (LocalRootMotionSource.IsValid() && (LocalRootMotionSource->LocalID == RootMotionIDMappings[MappingIndex].LocalID))
			{
				bFoundLocalSource = true;
				break;
			}
		}

		if (!bFoundLocalSource)
		{
			RootMotionIDMappings.RemoveAt(MappingIndex, EAllowShrinking::No);
		}
	}

	bool bDumpDebugInfo = false;

	// Root Motion Sources are applied independently on servers and clients.
	// FRootMotionSource::LocalID is an ID added when that Source is applied.
	// When we receive RootMotionSource data from the server, LocalIDs on that
	// RootMotion data are the server LocalIDs. When processing an FRootMotionSourceGroup
	// for use on clients, we want to map server LocalIDs to our LocalIDs.
	// We save off these mappings for quicker access and to save having to
	// "find best match" every time we receive server data.
	for (TSharedPtr<FRootMotionSource>& ServerRootMotionSource : InOutServerRootMotion.RootMotionSources)
	{
		if (ServerRootMotionSource.IsValid())
		{
			const uint16 ServerID = ServerRootMotionSource->LocalID;

			// Reset LocalID of replicated ServerRootMotionSource, and find a local match.
			ServerRootMotionSource->LocalID = (uint16)ERootMotionSourceID::Invalid;

			// See if we have any recent mappings that match this server ID
			// If we do, change it to that mapping and update the timestamp
			{
				bool bMappingFound = false;
				for (FRootMotionServerToLocalIDMapping& Mapping : RootMotionIDMappings)
				{
					if (ServerID == Mapping.ServerID)
					{
						ServerRootMotionSource->LocalID = Mapping.LocalID;
						Mapping.TimeStamp = TimeStamp;
						bMappingFound = true;
						break; // Found it, don't need to search any more mappings
					}
				}

				if (bMappingFound)
				{
					// We rely on this rule (Matches) being always true, so in non-shipping builds make sure it never breaks.
					for (const TSharedPtr<FRootMotionSource>& LocalRootMotionSource : LocalRootMotionToMatchWith.RootMotionSources)
					{
						if (LocalRootMotionSource.IsValid() && (LocalRootMotionSource->LocalID == ServerRootMotionSource->LocalID))
						{
							if (!LocalRootMotionSource->Matches(ServerRootMotionSource.Get()))
							{
								ensureMsgf(false,
									TEXT("Character(%s) Local RootMotionSource(%s) has the same LocalID(%d) as a non-matching ServerRootMotionSource(%s)!"),
									*GetNameSafe(CharacterOwner), *LocalRootMotionSource->ToSimpleString(), LocalRootMotionSource->LocalID, *ServerRootMotionSource->ToSimpleString());

								bDumpDebugInfo = true;
							}

							break;
						}
					}

					// We've found the correct LocalID, done with this one, process next ServerRootMotionSource
					continue;
				}
			}

			// If no mapping found, find match out of Local RootMotionSources that are not already mapped
			bool bMatchFound = false;
			TArray<TSharedPtr<FRootMotionSource>> LocalRootMotionSources;
			LocalRootMotionSources.Reserve(LocalRootMotionToMatchWith.RootMotionSources.Num() + LocalRootMotionToMatchWith.PendingAddRootMotionSources.Num());
			LocalRootMotionSources.Append(LocalRootMotionToMatchWith.RootMotionSources);
			LocalRootMotionSources.Append(LocalRootMotionToMatchWith.PendingAddRootMotionSources);
			for (const TSharedPtr<FRootMotionSource>& LocalRootMotionSource : LocalRootMotionSources)
			{
				if (LocalRootMotionSource.IsValid())
				{
					const uint16 LocalID = LocalRootMotionSource->LocalID;

					// Check if the LocalID is already mapped to a ServerID; if it's already "claimed",
					// it's not valid for being a match to our unmatched server source
					{
						bool bMappingFound = false;
						for (FRootMotionServerToLocalIDMapping& Mapping : RootMotionIDMappings)
						{
							if (LocalID == Mapping.LocalID)
							{
								bMappingFound = true;
								break; // Found it, don't need to search any more mappings
							}
						}

						if (bMappingFound)
						{
							continue; // We found a ServerID matching this LocalID, so we don't try to match this
						}
					}

					// This LocalRootMotionSource is a valid possible match to the ServerRootMotionSource
					if (LocalRootMotionSource->Matches(ServerRootMotionSource.Get()))
					{
						// We have a match!
						// Assign LocalID
						ServerRootMotionSource->LocalID = LocalID;

						// Add to Mapping
						{
							FRootMotionServerToLocalIDMapping NewMapping;
							NewMapping.LocalID = LocalID;
							NewMapping.ServerID = ServerID;
							NewMapping.TimeStamp = TimeStamp;

							RootMotionIDMappings.Add(NewMapping);
							bMatchFound = true;
							break; // Stop searching LocalRootMotionSources, we've found a match
						}
					}
				}
			} // loop through LocalRootMotionSources

			// if we don't find a match, set an invalid LocalID so that we know it's an invalid ID from the server
			// This doesn't mean it's a "bad" RootMotionSource; just that the Server sent a RootMotionSource
			// that we don't have in the current LocalRootMotion group we're searching. It's possible that next
			// frame the LocalRootMotionSource was added/will be added and from then on we'll match & correct from
			// the Server
			if (!bMatchFound)
			{
				ServerRootMotionSource->LocalID = (uint16)ERootMotionSourceID::Invalid;
			}
		}
	} // loop through ServerRootMotionSources

	if (bDumpDebugInfo)
	{
		UE_LOGF(LogRootMotion, Warning, "Dumping current mappings:");
		for (FRootMotionServerToLocalIDMapping& Mapping : RootMotionIDMappings)
		{
			UE_LOGF(LogRootMotion, Warning, "- LocalID(%d) ServerID(%d)", Mapping.LocalID, Mapping.ServerID);
		}

		UE_LOGF(LogRootMotion, Warning, "Dumping local RootMotionSources:");
		for (const TSharedPtr<FRootMotionSource>& LocalRootMotionSource : LocalRootMotionToMatchWith.RootMotionSources)
		{
			if (LocalRootMotionSource.IsValid())
			{
				UE_LOGF(LogRootMotion, Warning, "- LocalRootMotionSource(%ls)", *LocalRootMotionSource->ToSimpleString());
			}
		}

		UE_LOGF(LogRootMotion, Warning, "Dumping server RootMotionSources:");
		for (TSharedPtr<FRootMotionSource>& ServerRootMotionSource : InOutServerRootMotion.RootMotionSources)
		{
			if (ServerRootMotionSource.IsValid())
			{
				UE_LOGF(LogRootMotion, Warning, "- ServerRootMotionSource(%ls)", *ServerRootMotionSource->ToSimpleString());
			}
		}
	}
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS // For deprecated members of FNetworkPredictionData_Client_Character

FNetworkPredictionData_Client_Character::FNetworkPredictionData_Client_Character(const UCharacterMovementComponent& ClientMovement)
	: ClientUpdateRealTime(0.f)
	, CurrentTimeStamp(0.f)
	, LastReceivedAckRealTime(0.f)
	, PendingMove(NULL)
	, LastAckedMove(NULL)
	, MaxFreeMoveCount(96)
	, MaxSavedMoveCount(96)
	, bUpdatePosition(false)
	, OriginalMeshTranslationOffset(ForceInitToZero)
	, MeshTranslationOffset(ForceInitToZero)
	, OriginalMeshRotationOffset(FQuat::Identity)
	, MeshRotationOffset(FQuat::Identity)	
	, MeshRotationTarget(FQuat::Identity)
	, LastCorrectionDelta(0.f)
	, LastCorrectionTime(0.f)
	, MaxClientSmoothingDeltaTime(0.5f)
	, SmoothingServerTimeStamp(0.f)
	, SmoothingClientTimeStamp(0.f)
	, MaxSmoothNetUpdateDist(0.f)
	, NoSmoothNetUpdateDist(0.f)
	, SmoothNetUpdateTime(0.f)
	, SmoothNetUpdateRotationTime(0.f)
	, MaxMoveDeltaTime(0.125f)
	, LastSmoothLocation(FVector::ZeroVector)
	, LastServerLocation(FVector::ZeroVector)
	, SimulatedDebugDrawTime(0.0f)
	, DebugForcedPacketLossTimerStart(0.0f)
{
	MaxSmoothNetUpdateDist = ClientMovement.NetworkMaxSmoothUpdateDistance;
	NoSmoothNetUpdateDist = ClientMovement.NetworkNoSmoothUpdateDistance;

	const bool bIsListenServer = (ClientMovement.GetNetMode() == NM_ListenServer);
	SmoothNetUpdateTime = (bIsListenServer ? ClientMovement.ListenServerNetworkSimulatedSmoothLocationTime : ClientMovement.NetworkSimulatedSmoothLocationTime);
	SmoothNetUpdateRotationTime = (bIsListenServer ? ClientMovement.ListenServerNetworkSimulatedSmoothRotationTime : ClientMovement.NetworkSimulatedSmoothRotationTime);

	const AGameNetworkManager* GameNetworkManager = (const AGameNetworkManager*)(AGameNetworkManager::StaticClass()->GetDefaultObject());
	if (GameNetworkManager)
	{
		MaxMoveDeltaTime = GameNetworkManager->MaxMoveDeltaTime;
		MaxClientSmoothingDeltaTime = FMath::Max(GameNetworkManager->MaxClientSmoothingDeltaTime, MaxMoveDeltaTime * 2.0f);
	}

	if (ClientMovement.GetOwnerRole() == ROLE_AutonomousProxy)
	{
		SavedMoves.Reserve(MaxSavedMoveCount);
		FreeMoves.Reserve(MaxFreeMoveCount);
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS // For deprecated members of FNetworkPredictionData_Client_Character


FNetworkPredictionData_Client_Character::~FNetworkPredictionData_Client_Character()
{
	SavedMoves.Empty();
	FreeMoves.Empty();
	PendingMove = NULL;
	LastAckedMove = NULL;
}

void FNetworkPredictionData_Client_Character::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	Super::AddStructReferencedObjects(Collector);

	for (FSavedMovePtr SavedMove : SavedMoves)
	{
		if (const FSavedMove_Character* SavedMovePtr = SavedMove.Get())
		{
			SavedMovePtr->AddStructReferencedObjects(Collector);
		}
	}

	if (const FSavedMove_Character* PendingMovePtr = PendingMove.Get())
	{
		PendingMovePtr->AddStructReferencedObjects(Collector);
	}

	if (const FSavedMove_Character* LastAckedMovePtr = LastAckedMove.Get())
	{
		LastAckedMovePtr->AddStructReferencedObjects(Collector);
	}
}


FSavedMovePtr FNetworkPredictionData_Client_Character::CreateSavedMove()
{
	if (SavedMoves.Num() >= MaxSavedMoveCount)
	{
		UE_LOGF(LogNetPlayerMovement, Warning, "CreateSavedMove: Hit limit of %d saved moves (timing out or very bad ping?)", SavedMoves.Num());
		// Free all saved moves
		for (int32 i=0; i < SavedMoves.Num(); i++)
		{
			FreeMove(SavedMoves[i]);
		}
		SavedMoves.Reset();
	}

	if (FreeMoves.Num() == 0)
	{
		// No free moves, allocate a new one.
		FSavedMovePtr NewMove = AllocateNewMove();
		checkSlow(NewMove.IsValid());
		NewMove->Clear();
		return NewMove;
	}
	else
	{
		// Pull from the free pool
		FSavedMovePtr FirstFree = FreeMoves.Pop(EAllowShrinking::No);
		FirstFree->Clear();
		return FirstFree;
	}
}


FSavedMovePtr FNetworkPredictionData_Client_Character::AllocateNewMove()
{
	return FSavedMovePtr(new FSavedMove_Character());
}


void FNetworkPredictionData_Client_Character::FreeMove(const FSavedMovePtr& Move)
{
	if (Move.IsValid())
	{
		// Only keep a pool of a limited number of moves.
		if (FreeMoves.Num() < MaxFreeMoveCount)
		{
			FreeMoves.Push(Move);
		}

		// Shouldn't keep a reference to the move on the free list.
		if (PendingMove == Move)
		{
			PendingMove = NULL;
		}
		if( LastAckedMove == Move )
		{
			LastAckedMove = NULL;
		}
	}
}

int32 FNetworkPredictionData_Client_Character::GetSavedMoveIndex(float TimeStamp) const
{
	if( SavedMoves.Num() > 0 )
	{
		// If LastAckedMove isn't using an old TimeStamp (before reset), we can prevent the iteration if incoming TimeStamp is outdated
		if( LastAckedMove.IsValid() && !LastAckedMove->bOldTimeStampBeforeReset && (TimeStamp <= LastAckedMove->TimeStamp) )
		{
			return INDEX_NONE;
		}

		// Otherwise see if we can find this move.
		for (int32 Index=0; Index<SavedMoves.Num(); Index++)
		{
			const FSavedMove_Character* CurrentMove = SavedMoves[Index].Get();
			checkSlow(CurrentMove != nullptr);
			if( CurrentMove->TimeStamp == TimeStamp )
			{
				return Index;
			}
		}
	}
	return INDEX_NONE;
}

void FNetworkPredictionData_Client_Character::AckMove(int32 AckedMoveIndex, UCharacterMovementComponent& CharacterMovementComponent) 
{
	// It is important that we know the move exists before we go deleting outdated moves.
	// Timestamps are not guaranteed to be increasing order all the time, since they can be reset!
	if( AckedMoveIndex != INDEX_NONE )
	{
		// Keep reference to LastAckedMove
		const FSavedMovePtr& AckedMove = SavedMoves[AckedMoveIndex];
		UE_LOGF(LogNetPlayerMovement, VeryVerbose, "AckedMove Index: %2d (%2d moves). TimeStamp: %f, CurrentTimeStamp: %f", AckedMoveIndex, SavedMoves.Num(), AckedMove->TimeStamp, CurrentTimeStamp);
		if( LastAckedMove.IsValid() )
		{
			FreeMove(LastAckedMove);
		}
		LastAckedMove = AckedMove;

		// Free expired moves.
		for(int32 MoveIndex=0; MoveIndex<AckedMoveIndex; MoveIndex++)
		{
			const FSavedMovePtr& Move = SavedMoves[MoveIndex];
			FreeMove(Move);
		}

		// And finally cull all of those, so only the unacknowledged moves remain in SavedMoves.
		SavedMoves.RemoveAt(0, AckedMoveIndex + 1, EAllowShrinking::No);
	}

	if (const UWorld* const World = CharacterMovementComponent.GetWorld())
	{
		LastReceivedAckRealTime = World->GetRealTimeSeconds();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // For deprecated members of FNetworkPredictionData_Server_Character

FNetworkPredictionData_Server_Character::FNetworkPredictionData_Server_Character(const UCharacterMovementComponent& ServerMovement)
	: PendingAdjustment()
	, CurrentClientTimeStamp(0.f)
	, ServerAccumulatedClientTimeStamp(0.0)
	, LastUpdateTime(0.f)
	, ServerTimeStampLastServerMove(0.f)
	, MaxMoveDeltaTime(0.125f)
	, bForceClientUpdate(false)
	, LifetimeRawTimeDiscrepancy(0.f)
	, TimeDiscrepancy(0.f)
	, bResolvingTimeDiscrepancy(false)
	, TimeDiscrepancyResolutionMoveDeltaOverride(0.f)
	, TimeDiscrepancyAccumulatedClientDeltasSinceLastServerTick(0.f)
	, WorldCreationTime(0.f)
{
	const AGameNetworkManager* GameNetworkManager = (const AGameNetworkManager*)(AGameNetworkManager::StaticClass()->GetDefaultObject());
	if (GameNetworkManager)
	{
		MaxMoveDeltaTime = GameNetworkManager->MaxMoveDeltaTime;
		if (GameNetworkManager->MaxMoveDeltaTime > GameNetworkManager->MAXCLIENTUPDATEINTERVAL)
		{
			UE_LOGF(LogNetPlayerMovement, Warning, "GameNetworkManager::MaxMoveDeltaTime (%f) is greater than GameNetworkManager::MAXCLIENTUPDATEINTERVAL (%f)! Server will interfere with move deltas that large!", GameNetworkManager->MaxMoveDeltaTime, GameNetworkManager->MAXCLIENTUPDATEINTERVAL);
		}
	}

	const UWorld* World = ServerMovement.GetWorld();
	if (World)
	{
		WorldCreationTime = World->GetTimeSeconds();
		ServerTimeStamp = World->GetTimeSeconds();
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS // For deprecated members of FNetworkPredictionData_Server_Character


FNetworkPredictionData_Server_Character::~FNetworkPredictionData_Server_Character()
{
}


float FNetworkPredictionData_Server_Character::GetServerMoveDeltaTime(float ClientTimeStamp, float ActorTimeDilation) const
{
	if (bResolvingTimeDiscrepancy)
	{
		return TimeDiscrepancyResolutionMoveDeltaOverride;
	}
	else
	{
		return GetBaseServerMoveDeltaTime(ClientTimeStamp, ActorTimeDilation);
	}
}

float FNetworkPredictionData_Server_Character::GetBaseServerMoveDeltaTime(float ClientTimeStamp, float ActorTimeDilation) const
{
	// Allow server to run longer moves than the client is allowed to send, to avoid corrections when there is a gap in lost client moves.
	const float ServerScalar = FMath::Max(1.0f, CharacterMovementCVars::NetServerMaxMoveDeltaTimeScalar);
	const float DeltaTime = FMath::Min(ServerScalar * MaxMoveDeltaTime * ActorTimeDilation, ClientTimeStamp - CurrentClientTimeStamp);
	return DeltaTime;
}

void FNetworkPredictionData_Server_Character::Serialize(FArchive& Archive)
{
	if (Archive.IsMigratingRemoteObjects())
	{
		FNetworkPredictionData_Server::Serialize(Archive);

		PendingAdjustment.Serialize(Archive);

		Archive << CurrentClientTimeStamp;
		Archive << LastReceivedClientTimeStamp;
		Archive << ServerAccumulatedClientTimeStamp;
		Archive << LastUpdateTime;
		Archive << ServerTimeStampLastServerMove;
		Archive << MaxMoveDeltaTime;
		
		bool bTempForceClientUpdate = bForceClientUpdate;
		Archive.SerializeBits(&bTempForceClientUpdate, 1);
		bForceClientUpdate = bTempForceClientUpdate;

		Archive << LifetimeRawTimeDiscrepancy;
		Archive << TimeDiscrepancy;
		Archive.SerializeBits(&bResolvingTimeDiscrepancy, 1);
		Archive << TimeDiscrepancyResolutionMoveDeltaOverride;
		Archive << TimeDiscrepancyAccumulatedClientDeltasSinceLastServerTick;
		Archive << WorldCreationTime;
	}
};


FSavedMove_Character::FSavedMove_Character()
{
	AccelMagThreshold = 1.f;
	AccelDotThreshold = 0.9f;
	AccelDotThresholdCombine = 0.996f; // approx 5 degrees.
	MaxSpeedThresholdCombine = 10.0f;
}

FSavedMove_Character::~FSavedMove_Character()
{
}

void FSavedMove_Character::Clear()
{
	bPressedJump = false;
	bWantsToCrouch = false;
	bForceMaxAccel = false;
	bForceNoCombine = false;
	bOldTimeStampBeforeReset = false;
	bWasJumping = false;

	TimeStamp = 0.f;
	DeltaTime = 0.f;
	CustomTimeDilation = 1.0f;
	JumpKeyHoldTime = 0.0f;
	JumpForceTimeRemaining = 0.0f;
	JumpCurrentCount = 0;
	JumpMaxCount = 1;
	MovementMode = 0; // Deprecated, keep backwards compat until removed

	StartPackedMovementMode = 0;
	StartLocation = FVector::ZeroVector;
	StartRelativeLocation = FVector::ZeroVector;
	StartVelocity = FVector::ZeroVector;
	StartFloor = FFindFloorResult();
	StartRotation = FRotator::ZeroRotator;
	StartControlRotation = FRotator::ZeroRotator;
	StartBaseRotation = FQuat::Identity;
	StartCapsuleRadius = 0.f;
	StartCapsuleHalfHeight = 0.f;
	StartMovementBaseInterfaceData.Clear();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	StartBase = nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	StartBoneName = NAME_None;
	StartActorOverlapCounter = 0;
	StartComponentOverlapCounter = 0;

	StartAttachParent = nullptr;
	StartAttachSocketName = NAME_None;
	StartAttachRelativeLocation = FVector::ZeroVector;
	StartAttachRelativeRotation = FRotator::ZeroRotator;

	SavedLocation = FVector::ZeroVector;
	SavedRotation = FRotator::ZeroRotator;
	SavedRelativeLocation = FVector::ZeroVector;
	SavedControlRotation = FRotator::ZeroRotator;
	SavedRelativeAcceleration = FVector::ZeroVector;
	Acceleration = FVector::ZeroVector;
	MaxSpeed = 0.0f;
	AccelMag = 0.0f;
	AccelNormal = FVector::ZeroVector;

	EndMovementBaseInterfaceData.Clear();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EndBase = nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	EndBoneName = NAME_None;
	EndActorOverlapCounter = 0;
	EndComponentOverlapCounter = 0;
	EndPackedMovementMode = 0;

	EndAttachParent = nullptr;
	EndAttachSocketName = NAME_None;
	EndAttachRelativeLocation = FVector::ZeroVector;
	EndAttachRelativeRotation = FRotator::ZeroRotator;

	RootMotionMontage = NULL;
	RootMotionTrackPosition = 0.f;
	RootMotionPreviousTrackPosition = 0.f;
	RootMotionPlayRateWithScale = 1.f;
	RootMotionMovement.Clear();

	SavedRootMotion.Clear();
}


void FSavedMove_Character::SetMoveFor(ACharacter* Character, float InDeltaTime, FVector const& NewAccel, class FNetworkPredictionData_Client_Character & ClientData)
{
	CharacterOwner = Character;
	DeltaTime = InDeltaTime;
	
	SetInitialPosition(Character);

	AccelMag = NewAccel.Size();
	AccelNormal = (AccelMag > UE_SMALL_NUMBER ? NewAccel / AccelMag : FVector::ZeroVector);
	
	// Round value, so that client and server match exactly (and so we can send with less bandwidth). This rounded value is copied back to the client in ReplicateMoveToServer.
	// This is done after the AccelMag and AccelNormal are computed above, because those are only used client-side for combining move logic and need to remain accurate.
	Acceleration = Character->GetCharacterMovement()->RoundAcceleration(NewAccel);
	
	MaxSpeed = Character->GetCharacterMovement()->GetMaxSpeed();

	JumpCurrentCount = Character->JumpCurrentCountPreJump;
	bWantsToCrouch = Character->GetCharacterMovement()->bWantsToCrouch;
	bForceMaxAccel = Character->GetCharacterMovement()->bForceMaxAccel;
	StartPackedMovementMode = Character->GetCharacterMovement()->PackNetworkMovementMode();
	MovementMode = StartPackedMovementMode; // Deprecated, keep backwards compat until removed

	// Root motion source-containing moves should never be combined
	// Main discovered issue being a move without root motion combining with
	// a move with it will cause the DeltaTime for that next move to be larger than
	// intended (effectively root motion applies to movement that happened prior to its activation)
	if (Character->GetCharacterMovement()->CurrentRootMotion.HasActiveRootMotionSources())
	{
		bForceNoCombine = true;
	}

	// Moves with anim root motion should not be combined
	const FAnimMontageInstance* RootMotionMontageInstance = Character->GetRootMotionAnimMontageInstance();
	if (RootMotionMontageInstance)
	{
		bForceNoCombine = true;
	}

	// Launch velocity gives instant and potentially huge change of velocity
	// Avoid combining move to prevent from reverting locations until server catches up
	const bool bHasLaunchVelocity = !Character->GetCharacterMovement()->PendingLaunchVelocity.IsZero();
	if (bHasLaunchVelocity)
	{
		bForceNoCombine = true;
	}

	TimeStamp = ClientData.CurrentTimeStamp;
}

void FSavedMove_Character::SetInitialPosition(ACharacter* Character)
{
	StartLocation = Character->GetActorLocation();
	StartRotation = Character->GetActorRotation();
	StartVelocity = Character->GetCharacterMovement()->Velocity;
	StartMovementBaseInterfaceData = Character->GetBasedMovement().MovementBaseInterfaceData;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	StartBase = Character->GetBasedMovement().MovementBase;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	StartBaseRotation = FQuat::Identity;
	StartFloor = Character->GetCharacterMovement()->CurrentFloor;
	CustomTimeDilation = Character->CustomTimeDilation;
	StartBoneName = Character->GetBasedMovement().BoneName;
	StartActorOverlapCounter = Character->NumActorOverlapEventsCounter;
	StartComponentOverlapCounter = UPrimitiveComponent::GlobalOverlapEventsCounter;

	if (MovementBaseUtility::UseRelativeLocation(&StartMovementBaseInterfaceData))
	{
		StartRelativeLocation = Character->GetBasedMovement().Location;
		FVector StartBaseLocation_Unused;
		MovementBaseUtility::GetMovementBaseTransform(&StartMovementBaseInterfaceData, StartBoneName, StartBaseLocation_Unused, StartBaseRotation);
	}

	// Attachment state
	if (const USceneComponent* UpdatedComponent = Character->GetCharacterMovement()->UpdatedComponent)
	{
		StartAttachParent = UpdatedComponent->GetAttachParent();
		StartAttachSocketName = UpdatedComponent->GetAttachSocketName();
		StartAttachRelativeLocation = UpdatedComponent->GetRelativeLocation();
		StartAttachRelativeRotation = UpdatedComponent->GetRelativeRotation();
	}

	StartControlRotation = Character->GetControlRotation().Clamp();
	Character->GetCapsuleComponent()->GetScaledCapsuleSize(StartCapsuleRadius, StartCapsuleHalfHeight);

	// Jump state
	bPressedJump = Character->bPressedJump;
	bWasJumping = Character->bWasJumping;
	JumpKeyHoldTime = Character->JumpKeyHoldTime;
	JumpForceTimeRemaining = Character->JumpForceTimeRemaining;
	JumpMaxCount = Character->JumpMaxCount;
}

void FSavedMove_Character::PostUpdate(ACharacter* Character, FSavedMove_Character::EPostUpdateMode PostUpdateMode)
{
	// Common code for both recording and after a replay.
	{
		EndPackedMovementMode = Character->GetCharacterMovement()->PackNetworkMovementMode();
		MovementMode = EndPackedMovementMode; // Deprecated, keep backwards compat until removed
		SavedLocation = Character->GetActorLocation();
		SavedRotation = Character->GetActorRotation();
		SavedVelocity = Character->GetVelocity();
#if ENABLE_NAN_DIAGNOSTIC
		const float WarnVelocitySqr = 20000.f * 20000.f;
		if (SavedVelocity.SizeSquared() > WarnVelocitySqr)
		{
			if (Character->SavedRootMotion.HasActiveRootMotionSources())
			{
				UE_LOGF(LogCharacterMovement, Log, "FSavedMove_Character::PostUpdate detected very high Velocity! (%ls), but with active root motion sources (could be intentional)", *SavedVelocity.ToString());
			}
			else
			{
				UE_LOGF(LogCharacterMovement, Warning, "FSavedMove_Character::PostUpdate detected very high Velocity! (%ls)", *SavedVelocity.ToString());
			}
		}
#endif
		// Movement base and base-relative movement
		EndMovementBaseInterfaceData = Character->GetBasedMovement().MovementBaseInterfaceData;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EndBase = Character->GetBasedMovement().MovementBase;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		EndBoneName = Character->GetBasedMovement().BoneName;
		if (MovementBaseUtility::UseRelativeLocation(&EndMovementBaseInterfaceData))
		{
			SavedRelativeLocation = Character->GetBasedMovement().Location;
		}

		// Save off movement base-relative acceleration if needed
		if (CharacterMovementCVars::NetUseBaseRelativeAcceleration && MovementBaseUtility::IsDynamicBase(&EndMovementBaseInterfaceData))
		{
			MovementBaseUtility::TransformDirectionToLocal(&EndMovementBaseInterfaceData, EndBoneName, Acceleration, SavedRelativeAcceleration);
		}

		// Attachment state
		if (const USceneComponent* UpdatedComponent = Character->GetCharacterMovement()->UpdatedComponent)
		{
			EndAttachParent = UpdatedComponent->GetAttachParent();
			EndAttachSocketName = UpdatedComponent->GetAttachSocketName();
			EndAttachRelativeLocation = UpdatedComponent->GetRelativeLocation();
			EndAttachRelativeRotation = UpdatedComponent->GetRelativeRotation();
		}

		SavedControlRotation = Character->GetControlRotation().Clamp();
		if (!Character->GetController())
		{
			if (AController* ControllerOwner = Cast<AController>(Character->GetOwner()))
			{
				SavedControlRotation = ControllerOwner->GetControlRotation().Clamp();
			}
		}
	}

	if (PostUpdateMode == PostUpdate_Record)
	{
		// Only save RootMotion params when initially recording
		const FAnimMontageInstance* RootMotionMontageInstance = Character->GetRootMotionAnimMontageInstance();
		if (RootMotionMontageInstance)
		{
			if (!RootMotionMontageInstance->IsRootMotionDisabled())
			{
				RootMotionMontage = RootMotionMontageInstance->Montage;
				RootMotionTrackPosition = RootMotionMontageInstance->GetPosition();
				RootMotionPreviousTrackPosition = RootMotionMontageInstance->GetPreviousPosition();
				RootMotionPlayRateWithScale = RootMotionMontageInstance->GetPlayRate() * RootMotionMontage->RateScale;
				RootMotionMovement = Character->ClientRootMotionParams;
			}

			// Moves where anim root motion is being played should not be combined
			bForceNoCombine = true;
		}

		// Save off Root Motion Sources
		if( Character->SavedRootMotion.HasActiveRootMotionSources() )
		{
			SavedRootMotion = Character->SavedRootMotion;
			bForceNoCombine = true;
		}

		// Don't want to combine moves that trigger overlaps, because by moving back and replaying the move we could retrigger overlaps.
		EndActorOverlapCounter = Character->NumActorOverlapEventsCounter;
		EndComponentOverlapCounter = UPrimitiveComponent::GlobalOverlapEventsCounter;
		if ((StartActorOverlapCounter != EndActorOverlapCounter) || (StartComponentOverlapCounter != EndComponentOverlapCounter))
		{
			bForceNoCombine = true;
		}

		// Don't combine or delay moves where velocity changes to/from zero.
		if (StartVelocity.IsZero() != SavedVelocity.IsZero())
		{
			bForceNoCombine = true;
		}

		// Don't combine if this move caused us to change movement modes during the move.
		if (StartPackedMovementMode != EndPackedMovementMode)
		{
			bForceNoCombine = true;
		}

		// Don't combine when jump input just began or ended during the move.
		if (bPressedJump != CharacterOwner->bPressedJump)
		{
			bForceNoCombine = true;
		}
	}
	else if (PostUpdateMode == PostUpdate_Replay)
	{
		if( Character->bClientResimulateRootMotionSources )
		{
			// When replaying moves, the next move should use the results of this move
			// so that future replayed moves account for the server correction
			Character->SavedRootMotion = Character->GetCharacterMovement()->CurrentRootMotion;
		}
	}
}

bool FSavedMove_Character::IsImportantMove(const FSavedMovePtr& LastAckedMovePtr) const
{
	const FSavedMove_Character* LastAckedMove = LastAckedMovePtr.Get();

	// Check if any important movement flags have changed status.
	if (GetCompressedFlags() != LastAckedMove->GetCompressedFlags())
	{
		return true;
	}

	if (StartPackedMovementMode != LastAckedMove->EndPackedMovementMode)
	{
		return true;
	}

	if (EndPackedMovementMode != LastAckedMove->EndPackedMovementMode)
	{
		return true;
	}

	// check if acceleration has changed significantly
	if (Acceleration != LastAckedMove->Acceleration)
	{
		// Compare magnitude and orientation
		if( (FMath::Abs(AccelMag - LastAckedMove->AccelMag) > AccelMagThreshold) || ((AccelNormal | LastAckedMove->AccelNormal) < AccelDotThreshold) )
		{
			return true;
		}
	}
	return false;
}

FVector FSavedMove_Character::GetRevertedLocation() const
{
	if (const USceneComponent* AttachParent = StartAttachParent.Get())
	{
		return AttachParent->GetSocketTransform(StartAttachSocketName).TransformPosition(StartAttachRelativeLocation);
	}

	if (MovementBaseUtility::UseRelativeLocation(&StartMovementBaseInterfaceData))
	{
		FVector WorldSpacePosition;
		MovementBaseUtility::TransformLocationToWorld(&StartMovementBaseInterfaceData, StartBoneName, StartRelativeLocation, WorldSpacePosition);
		return WorldSpacePosition;
	}

	return StartLocation;
}

bool UCharacterMovementComponent::CanDelaySendingMove(const FSavedMovePtr& NewMovePtr)
{
	const FSavedMove_Character* NewMove = NewMovePtr.Get();

	// Don't delay moves that change movement mode over the course of the move.
	if (NewMove->StartPackedMovementMode != NewMove->EndPackedMovementMode)
	{
		return false;
	}

	// If we know we don't want to combine this move, reduce latency and avoid misprediction by flushing immediately.
	if (NewMove->bForceNoCombine)
	{
		return false;
	}

	return true;
}

float UCharacterMovementComponent::GetClientNetSendDeltaTime(const APlayerController* PC, const FNetworkPredictionData_Client_Character* ClientData, const FSavedMovePtr& NewMove) const
{
	const UPlayer* Player = (PC ? ToRawPtr(PC->Player) : nullptr);
	const UWorld* MyWorld = GetWorld();
	const AGameStateBase* const GameState = MyWorld->GetGameState();
	const AGameNetworkManager* GameNetworkManager = (const AGameNetworkManager*)(AGameNetworkManager::StaticClass()->GetDefaultObject());
	float NetMoveDelta = GameNetworkManager->ClientNetSendMoveDeltaTime;

	if (PC && Player)
	{
		// send moves more frequently in small games where server isn't likely to be saturated
		if ((Player->CurrentNetSpeed > GameNetworkManager->ClientNetSendMoveThrottleAtNetSpeed) && (GameState != nullptr) && (GameState->PlayerArray.Num() <= GameNetworkManager->ClientNetSendMoveThrottleOverPlayerCount))
		{
			NetMoveDelta = GameNetworkManager->ClientNetSendMoveDeltaTime;
		}
		else
		{
			NetMoveDelta = FMath::Max(GameNetworkManager->ClientNetSendMoveDeltaTimeThrottled, 2 * GameNetworkManager->MoveRepSize / Player->CurrentNetSpeed);
		}

		// Lower frequency for standing still and not rotating camera
		if (Acceleration.IsZero() && Velocity.IsZero() && ClientData->LastAckedMove.IsValid() && ClientData->LastAckedMove->IsMatchingStartControlRotation(PC))
		{
			NetMoveDelta = FMath::Max(GameNetworkManager->ClientNetSendMoveDeltaTimeStationary, NetMoveDelta);
		}
	}
	
	return NetMoveDelta;
}

bool FSavedMove_Character::IsMatchingStartControlRotation(const APlayerController* PC) const
{
	return PC ? StartControlRotation.Equals(PC->GetControlRotation(), CharacterMovementCVars::NetStationaryRotationTolerance) : false;
}

void FSavedMove_Character::GetPackedAngles(uint32& YawAndPitchPack, uint8& RollPack) const
{
	// Compress rotation down to 5 bytes
	YawAndPitchPack = UCharacterMovementComponent::PackYawAndPitchTo32(SavedControlRotation.Yaw, SavedControlRotation.Pitch);
	RollPack = FRotator::CompressAxisToByte(SavedControlRotation.Roll);
}

bool FSavedMove_Character::CanCombineWith(const FSavedMovePtr& NewMovePtr, ACharacter* Character, float MaxDelta) const
{
	const FSavedMove_Character* NewMove = NewMovePtr.Get();

	if (bForceNoCombine || NewMove->bForceNoCombine)
	{
		return false;
	}

	if (bOldTimeStampBeforeReset)
	{
		return false;
	}

	// Cannot combine moves which contain root motion for now.
	// @fixme laurent - we should be able to combine most of them though, but current scheme of resetting pawn location and resimulating forward doesn't work.
	// as we don't want to tick montage twice (so we don't fire events twice). So we need to rearchitecture this so we tick only the second part of the move, and reuse the first part.
	if( (RootMotionMontage != NULL) || (NewMove->RootMotionMontage != NULL) )
	{
		return false;
	}

	if (NewMove->Acceleration.IsZero())
	{
		if (!Acceleration.IsZero())
		{
			return false;
		}
	}
	else
	{
		if (NewMove->DeltaTime + DeltaTime >= MaxDelta)
		{
			return false;
		}

		if (!FVector::Coincident(AccelNormal, NewMove->AccelNormal, AccelDotThresholdCombine))
		{
			return false;
		}	
	}

	// Don't combine moves where velocity changes to zero or from zero.
	if (StartVelocity.IsZero() != NewMove->StartVelocity.IsZero())
	{
		return false;
	}

	if (!FMath::IsNearlyEqual(MaxSpeed, NewMove->MaxSpeed, MaxSpeedThresholdCombine))
	{
		return false;
	}

	if ((MaxSpeed == 0.0f) != (NewMove->MaxSpeed == 0.0f))
	{
		return false;
	}

	// Don't combine on changes to/from zero JumpKeyHoldTime.
	if ((JumpKeyHoldTime == 0.f) != (NewMove->JumpKeyHoldTime == 0.f))
	{
		return false;
	}

	if ((bWasJumping != NewMove->bWasJumping) || (JumpCurrentCount != NewMove->JumpCurrentCount) || (JumpMaxCount != NewMove->JumpMaxCount))
	{
		return false;
	}
	
	// Don't combine on changes to/from zero.
	if ((JumpForceTimeRemaining == 0.f) != (NewMove->JumpForceTimeRemaining == 0.f))
	{
		return false;
	}
	
	// Compressed flags not equal, can't combine. This covers jump and crouch as well as any custom movement flags from overrides.
	if (GetCompressedFlags() != NewMove->GetCompressedFlags())
	{
		return false;
	}

	const FMovementBaseInterfaceData* OldMovementBaseInterfaceData = &StartMovementBaseInterfaceData;
	const FMovementBaseInterfaceData* NewMovementBaseInterfaceData = &NewMove->StartMovementBaseInterfaceData;
	const bool bDynamicBaseOld = MovementBaseUtility::IsDynamicBase(OldMovementBaseInterfaceData);
	const bool bDynamicBaseNew = MovementBaseUtility::IsDynamicBase(NewMovementBaseInterfaceData);

	// Change between static/dynamic requires separate moves (position sent as world vs relative)
	if (bDynamicBaseOld != bDynamicBaseNew)
	{
		return false;
	}

	// Only need to prevent combining when on a dynamic base that changes (unless forced off via CVar). Again, because relative location can change.
	const bool bPreventOnStaticBaseChange = (CharacterMovementCVars::NetEnableMoveCombiningOnStaticBaseChange == 0);
	if (bPreventOnStaticBaseChange || (bDynamicBaseOld || bDynamicBaseNew))
	{
		if (OldMovementBaseInterfaceData != NewMovementBaseInterfaceData)
		{
			return false;
		}

		if (StartBoneName != NewMove->StartBoneName)
		{
			return false;
		}
	}

	if (StartPackedMovementMode != NewMove->StartPackedMovementMode)
	{
		return false;
	}

	if (EndPackedMovementMode != NewMove->StartPackedMovementMode)
	{
		return false;
	}

	if (StartCapsuleRadius != NewMove->StartCapsuleRadius)
	{
		return false;
	}

	if (StartCapsuleHalfHeight != NewMove->StartCapsuleHalfHeight)
	{
		return false;
	}

	// No combining if attach parent changed.
	const USceneComponent* OldStartAttachParent = StartAttachParent.Get();
	const USceneComponent* OldEndAttachParent = EndAttachParent.Get();
	const USceneComponent* NewStartAttachParent = NewMove->StartAttachParent.Get();
	if (OldStartAttachParent != NewStartAttachParent || OldEndAttachParent != NewStartAttachParent)
	{
		return false;
	}

	// No combining if attach socket changed.
	if (StartAttachSocketName != NewMove->StartAttachSocketName || EndAttachSocketName != NewMove->StartAttachSocketName)
	{
		return false;
	}

	if (NewStartAttachParent != nullptr)
	{
		// If attached, no combining if relative location changed.
		const FVector RelativeLocationDelta = (StartAttachRelativeLocation - NewMove->StartAttachRelativeLocation);
		if (!RelativeLocationDelta.IsNearlyZero(CharacterMovementCVars::NetMoveCombiningAttachedLocationTolerance))
		{
			//UE_LOGF(LogCharacterMovement, Warning, "NoCombine: DeltaLocation(%ls)", *RelativeLocationDelta.ToString());
			return false;
		}
		// For rotation, Yaw doesn't matter for capsules
		FRotator RelativeRotationDelta = StartAttachRelativeRotation - NewMove->StartAttachRelativeRotation;
		RelativeRotationDelta.Yaw = 0.0f;
		if (!RelativeRotationDelta.IsNearlyZero(CharacterMovementCVars::NetMoveCombiningAttachedRotationTolerance))
		{
			return false;
		}
	}
	else
	{
		// Not attached to anything. Only combine if base hasn't rotated.
		if (!StartBaseRotation.Equals(NewMove->StartBaseRotation))
		{
			return false;
		}
	}

	if (CustomTimeDilation != NewMove->CustomTimeDilation)
	{
		return false;
	}

	// Don't combine moves with overlap event changes, since reverting back and then moving forward again can cause event spam.
	// This catches events between movement updates; moves that trigger events already set bForceNoCombine to false.
	if (EndActorOverlapCounter != NewMove->StartActorOverlapCounter)
	{
		return false;
	}

	return true;
}

void FSavedMove_Character::CombineWith(const FSavedMove_Character* OldMove, ACharacter* InCharacter, APlayerController* PC, const FVector& OldStartLocation)
{
	UCharacterMovementComponent* CharMovement = InCharacter->GetCharacterMovement();

	// to combine move, first revert pawn position to PendingMove start position, before playing combined move on client
	if (const USceneComponent* AttachParent = StartAttachParent.Get())
	{
		CharMovement->UpdatedComponent->SetRelativeLocationAndRotation(StartAttachRelativeLocation, StartAttachRelativeRotation, false, nullptr, CharMovement->GetTeleportType());
	}
	else
	{
		CharMovement->UpdatedComponent->SetWorldLocationAndRotation(OldStartLocation, OldMove->StartRotation, false, nullptr, CharMovement->GetTeleportType());
	}
	
	CharMovement->Velocity = OldMove->StartVelocity;

	FMovementBaseInterfaceData OldMovementBaseInterfaceData = OldMove->StartMovementBaseInterfaceData;
	CharMovement->SetBase(&OldMovementBaseInterfaceData, OldMove->StartBoneName);
	CharMovement->CurrentFloor = OldMove->StartFloor;

	// Now that we have reverted to the old position, prepare a new move from that position,
	// using our current velocity, acceleration, and rotation, but applied over the combined time from the old and new move.

	// Combine times for both moves
	DeltaTime += OldMove->DeltaTime;

	// Roll back jump force counters. SetInitialPosition() below will copy them to the saved move.
	InCharacter->JumpForceTimeRemaining = OldMove->JumpForceTimeRemaining;
	InCharacter->JumpKeyHoldTime = OldMove->JumpKeyHoldTime;
	InCharacter->JumpCurrentCountPreJump = OldMove->JumpCurrentCount;
}

void FSavedMove_Character::PrepMoveFor(ACharacter* Character)
{
	if( RootMotionMontage != NULL )
	{
		// If we need to resimulate Root Motion, then do so.
		if( Character->bClientResimulateRootMotion )
		{
			// Make sure RootMotion montage matches what we are playing now.
			FAnimMontageInstance * RootMotionMontageInstance = Character->GetRootMotionAnimMontageInstance();
			if( RootMotionMontageInstance && (RootMotionMontage == RootMotionMontageInstance->Montage) )
			{
				RootMotionMovement.Clear();
				RootMotionTrackPosition = RootMotionMontageInstance->GetPosition();
				RootMotionPreviousTrackPosition = RootMotionTrackPosition;
				RootMotionPlayRateWithScale = RootMotionMontageInstance->GetPlayRate() * RootMotionMontage->RateScale;
				RootMotionMontageInstance->SimulateAdvance(DeltaTime, RootMotionTrackPosition, RootMotionMovement);
				RootMotionMontageInstance->SetPosition(RootMotionTrackPosition);
				RootMotionMovement.ScaleRootMotionTranslation(Character->GetAnimRootMotionTranslationScale());
			}
		}

		// Restore root motion to that of this SavedMove to be used during replaying the Move
		Character->GetCharacterMovement()->RootMotionParams = RootMotionMovement;
	}

	// Resimulate Root Motion Sources if we need to - occurs after server RPCs over a correction during root motion sources.
	if( SavedRootMotion.HasActiveRootMotionSources() || Character->SavedRootMotion.HasActiveRootMotionSources() )
	{
		if( Character->bClientResimulateRootMotionSources )
		{
			// Note: This may need to change to a SimulatePrepare() that doesn't depend on everything
			// being "currently active" - if we have sources that are no longer around or valid,
			// we're not able to properly re-prepare them, and should just keep whatever we currently have

			// Apply any corrections/state from either last played move or last received from server (in ACharacter::SavedRootMotion)
			UE_LOGF(LogRootMotion, VeryVerbose, "SavedMove SavedRootMotion getting updated for SavedMove replays: %ls", *Character->GetName());
			SavedRootMotion.UpdateStateFrom(Character->SavedRootMotion);
			SavedRootMotion.CleanUpInvalidRootMotion(DeltaTime, *Character, *Character->GetCharacterMovement());
			SavedRootMotion.PrepareRootMotion(DeltaTime, *Character, *Character->GetCharacterMovement());
		}
		else
		{
			// If this is not the first SavedMove we are replaying, clean up any currently applied root motion sources so that if they have
			// SetVelocity/ClampVelocity finish settings they get applied correctly before CurrentRootMotion gets stomped below
			if (FNetworkPredictionData_Client_Character* ClientData = Character->GetCharacterMovement()->GetPredictionData_Client_Character())
			{
				if (ClientData->SavedMoves[0].Get() != this)
				{
					Character->GetCharacterMovement()->CurrentRootMotion.CleanUpInvalidRootMotion(DeltaTime, *Character, *Character->GetCharacterMovement());
				}
			}
		}

		// Restore root motion to that of this SavedMove to be used during replaying the Move
		Character->GetCharacterMovement()->CurrentRootMotion = SavedRootMotion;
	}

	Character->GetCharacterMovement()->bForceMaxAccel = bForceMaxAccel;

	if (CharacterMovementCVars::bResetJumpStateDuringResimIfNotFalling && !Character->GetCharacterMovement()->IsFalling())
	{
		Character->ResetJumpState();
	}
	else
	{
		Character->bWasJumping = bWasJumping;
		Character->JumpKeyHoldTime = JumpKeyHoldTime;
		Character->JumpForceTimeRemaining = JumpForceTimeRemaining;
		Character->JumpMaxCount = JumpMaxCount;
		Character->JumpCurrentCount = JumpCurrentCount;
		Character->JumpCurrentCountPreJump = JumpCurrentCount;
	}
	
	StartPackedMovementMode = Character->GetCharacterMovement()->PackNetworkMovementMode();
}


uint8 FSavedMove_Character::GetCompressedFlags() const
{
	uint8 Result = 0;

	if (bPressedJump)
	{
		Result |= FLAG_JumpPressed;
	}

	if (bWantsToCrouch)
	{
		Result |= FLAG_WantsToCrouch;
	}

	return Result;
}


void FSavedMove_Character::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	Collector.AddReferencedObject(RootMotionMontage);
	SavedRootMotion.AddStructReferencedObjects(Collector);
}


void UCharacterMovementComponent::UpdateFromCompressedFlags(uint8 Flags)
{
	if (!CharacterOwner)
	{
		return;
	}

	const bool bWasPressingJump = CharacterOwner->bPressedJump;

	CharacterOwner->bPressedJump = ((Flags & FSavedMove_Character::FLAG_JumpPressed) != 0);
	bWantsToCrouch = ((Flags & FSavedMove_Character::FLAG_WantsToCrouch) != 0);

	// Detect change in jump press on the server
	if (CharacterOwner->GetLocalRole() == ROLE_Authority) 
	{
		const bool bIsPressingJump = CharacterOwner->bPressedJump;
		if (bIsPressingJump && !bWasPressingJump)
		{
			CharacterOwner->Jump();
		}
		else if (!bIsPressingJump)
		{
			CharacterOwner->StopJumping();
		}
	}
}

void UCharacterMovementComponent::FlushServerMoves()
{
	// Send pendingMove to server if this character is replicating movement
	if (CharacterOwner && CharacterOwner->IsReplicatingMovement() && (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy))
	{
		FNetworkPredictionData_Client_Character* ClientData = GetPredictionData_Client_Character();
		if (!ClientData)
		{
			return;
		}

		if (ClientData->PendingMove.IsValid())
		{
			const UWorld* MyWorld = GetWorld();
			ClientData->ClientUpdateRealTime = MyWorld->GetRealTimeSeconds();

			FSavedMovePtr NewMove = ClientData->PendingMove;
			ClientData->PendingMove = nullptr;

			UE_CLOGF(CharacterOwner && UpdatedComponent, LogNetPlayerMovement, Verbose, "ClientMove (Flush) Time %f Acceleration %ls Velocity %ls Position %ls DeltaTime %f Mode %ls MovementBase %ls.%ls (Dynamic:%d) DualMove? %d",
				NewMove->TimeStamp, *NewMove->Acceleration.ToString(), *Velocity.ToString(), *UpdatedComponent->GetComponentLocation().ToString(), NewMove->DeltaTime, *GetMovementName(),
				*GetNameSafe(NewMove->EndMovementBaseInterfaceData.GetMovementBaseObject()), *NewMove->EndBoneName.ToString(), MovementBaseUtility::IsDynamicBase(&NewMove->EndMovementBaseInterfaceData) ? 1 : 0, ClientData->PendingMove.IsValid() ? 1 : 0);

			if (ShouldUsePackedMovementRPCs())
			{
				CallServerMovePacked(NewMove.Get(), nullptr, nullptr);
			}
			else
			{
				CallServerMove(NewMove.Get(), nullptr);
			}
		}
	}
}

ETeleportType UCharacterMovementComponent::GetTeleportType() const
{ 
	return bJustTeleported || bNetworkLargeClientCorrection ? ETeleportType::TeleportPhysics : ETeleportType::None;
}

void UCharacterMovementComponent::AccumulateRootMotionForAsync(float DeltaSeconds, FRootMotionAsyncData& RootMotion)
{
	SCOPE_CYCLE_COUNTER(STAT_CharacterMovementPerformMovement);

	const UWorld* MyWorld = GetWorld();
	if (!HasValidData() || MyWorld == nullptr)
	{
		return;
	}

	check(CharacterOwner && CharacterOwner->GetMesh());
	const bool bIsPlayingRootMotion = CharacterOwner->IsPlayingRootMotion();

	RootMotion.bHasAnimRootMotion |= HasAnimRootMotion();

	// no movement if we can't move, or if currently doing physical simulation on UpdatedComponent
	if (MovementMode == MOVE_None || UpdatedComponent->Mobility != EComponentMobility::Movable || UpdatedComponent->IsSimulatingPhysics())
	{
		if (!CharacterOwner->bClientUpdating && !CharacterOwner->bServerMoveIgnoreRootMotion)
		{
			// Consume root motion
			if (bIsPlayingRootMotion && CharacterOwner->GetMesh())
			{
				RootMotionParams.Clear();
			}
			if (CurrentRootMotion.HasActiveRootMotionSources())
			{
				CurrentRootMotion.Clear();
			}
		}
		RootMotion.TimeAccumulated += DeltaSeconds;
		return;
	}

	bool bHasRootMotion = RootMotionParams.bHasRootMotion;

	{
		// Clean up invalid RootMotion Sources.
		// This includes RootMotion sources that ended naturally.
		// They might want to perform a clamp on velocity or an override, 
		// so we want this to happen before ApplyAccumulatedForces and HandlePendingLaunch as to not clobber these.
		const bool bHasRootMotionSources = HasRootMotionSources();
		if (bHasRootMotionSources && !CharacterOwner->bClientUpdating && !CharacterOwner->bServerMoveIgnoreRootMotion)
		{
			const FVector VelocityBeforeCleanup = Velocity;
			CurrentRootMotion.CleanUpInvalidRootMotion(DeltaSeconds, *CharacterOwner, *this);
		}

		// Prepare Root Motion (generate/accumulate from root motion sources to be used later)
		if (bHasRootMotionSources && !CharacterOwner->bClientUpdating && !CharacterOwner->bServerMoveIgnoreRootMotion)
		{
			// Animation root motion - If using animation RootMotion, tick animations before running physics.
			if (bIsPlayingRootMotion && CharacterOwner->GetMesh())
			{
				//TickCharacterPose(DeltaSeconds);// doing this in Tick now

				// Make sure animation didn't trigger an event that destroyed us
				if (!HasValidData())
				{
					return;
				}

				// For local human clients, save off root motion data so it can be used by movement networking code.
				if (CharacterOwner->IsLocallyControlled() && (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy) && CharacterOwner->IsPlayingNetworkedRootMotionMontage())
				{
					CharacterOwner->ClientRootMotionParams = RootMotionParams;
				}
			}

			// Generates root motion to be used this frame from sources other than animation
			{
				CurrentRootMotion.PrepareRootMotion(DeltaSeconds, *CharacterOwner, *this, true);
			}

			// For local human clients, save off root motion data so it can be used by movement networking code.
			if (CharacterOwner->IsLocallyControlled() && (CharacterOwner->GetLocalRole() == ROLE_AutonomousProxy))
			{
				CharacterOwner->SavedRootMotion = CurrentRootMotion;
			}
		}

		RootMotion.bHasOverrideRootMotion |= CurrentRootMotion.HasOverrideVelocity();
		RootMotion.bHasOverrideWithIgnoreZAccumulate |= CurrentRootMotion.HasOverrideVelocityWithIgnoreZAccumulate();
		RootMotion.bHasAdditiveRootMotion |= CurrentRootMotion.HasAdditiveVelocity();
		RootMotion.bUseSensitiveLiftoff |= CurrentRootMotion.LastAccumulatedSettings.HasFlag(ERootMotionSourceSettingsFlags::UseSensitiveLiftoffCheck);

		// Apply Root Motion to Velocity
		if (CurrentRootMotion.HasOverrideVelocity() || bHasRootMotion)
		{
			// Animation root motion overrides Velocity and currently doesn't allow any other root motion sources
			if (bHasRootMotion)
			{
				// Convert to world space (animation root motion is always local)
				USkeletalMeshComponent* SkelMeshComp = CharacterOwner->GetMesh();
				if (SkelMeshComp)
				{
					// Convert Local Space Root Motion to world space. Do it right before used by physics to make sure we use up to date transforms, as translation is relative to rotation.
					RootMotionParams.Set(ConvertLocalRootMotionToWorld(RootMotionParams.GetRootMotionTransform(), DeltaSeconds));
					RootMotion.AnimTransform.Accumulate(RootMotionParams.GetRootMotionTransform());
				}
			}
			else
			{
				// We don't have animation root motion so we apply other sources
				if (DeltaSeconds > 0.f)
				{
					SCOPE_CYCLE_COUNTER(STAT_CharacterMovementRootMotionSourceApply);

					const FVector VelocityBeforeOverride = Velocity;
					FVector NewVelocity = Velocity;
					CurrentRootMotion.AccumulateOverrideRootMotionVelocity(DeltaSeconds, *CharacterOwner, *this, NewVelocity);
					if (RootMotion.TimeAccumulated == 0.f)
					{
						RootMotion.OverrideVelocity = NewVelocity;
					}
					else
					{
						// weighted average
						RootMotion.OverrideVelocity = ((RootMotion.OverrideVelocity * RootMotion.TimeAccumulated) + NewVelocity * DeltaSeconds) / (RootMotion.TimeAccumulated + DeltaSeconds);
					}
				}
			}

			// Root Motion has been used, clear
			RootMotionParams.Clear();
		}

		// NaN tracking
		devCode(ensureMsgf(!Velocity.ContainsNaN(), TEXT("UCharacterMovementComponent::PerformMovement: Velocity contains NaN (%s)\n%s"), *GetPathNameSafe(this), *Velocity.ToString()));

		// Apply Root Motion rotation after movement is complete.
		if (bHasRootMotion)
		{
			//const FQuat OldActorRotationQuat = UpdatedComponent->GetComponentQuat();
			//const FQuat RootMotionRotationQuat = RootMotionTransform.GetRotation();
			//if (!RootMotionRotationQuat.IsIdentity())
			//{
			//	const FQuat NewActorRotationQuat = RootMotionRotationQuat * OldActorRotationQuat;
			//	MoveUpdatedComponent(FVector::ZeroVector, NewActorRotationQuat, true);// WARNING
			//}

		}
		else if (CurrentRootMotion.HasActiveRootMotionSources())
		{
			FQuat RootMotionRotationQuat;
			if (CharacterOwner && UpdatedComponent && CurrentRootMotion.GetOverrideRootMotionRotation(DeltaSeconds, *CharacterOwner, *this, RootMotionRotationQuat))
			{
				RootMotion.OverrideRotation = RootMotionRotationQuat * RootMotion.OverrideRotation;
			}
		}
	} // End scoped movement update

	RootMotion.TimeAccumulated += DeltaSeconds;
}

void UCharacterMovementComponent::FillAsyncInput(const FVector& InputVector, FCharacterMovementComponentAsyncInput& AsyncInput)
{
	if (!CharacterOwner || !CharacterOwner->GetController())
	{
		return;
	}

	ensure(UpdatedComponent); // assumed to exist in  MockMoveUpdatedComponent
	ensure(CharacterOwner->GetCapsuleComponent()); // check in FindFloor
	ensure(!bRunPhysicsWithNoController);
	ensure(UpdatedComponent->GetOwner()); // assumed in ResolvePenetration
	ensure(GetOwner()); 

	// here because it can trigger other animation stuff (Mantis)
	UpdateCharacterStateBeforeMovement(AsyncRootMotion.TimeAccumulated);

	AsyncInput.bInitialized = true;

	AsyncInput.InputVector = InputVector;
	AsyncInput.NetworkSmoothingMode = NetworkSmoothingMode;
	AsyncInput.bIsNetModeClient = IsNetMode(NM_Client);
	AsyncInput.bWasSimulatingRootMotion = bWasSimulatingRootMotion;
	AsyncInput.bRunPhysicsWithNoController = bRunPhysicsWithNoController;
	AsyncInput.bForceMaxAccel = bForceMaxAccel;
	AsyncInput.MaxAcceleration = GetMaxAcceleration();
	AsyncInput.MinAnalogWalkSpeed = GetMinAnalogSpeed();
	AsyncInput.bIgnoreBaseRotation = bIgnoreBaseRotation;
	AsyncInput.bOrientRotationToMovement = bOrientRotationToMovement;
	AsyncInput.bUseControllerDesiredRotation = bUseControllerDesiredRotation;
	AsyncInput.bConstrainToPlane = bConstrainToPlane;
	AsyncInput.PlaneConstraintOrigin = PlaneConstraintOrigin;
	AsyncInput.PlaneConstraintNormal = PlaneConstraintNormal;
	AsyncInput.bHasValidData = HasValidData();
	AsyncInput.MaxStepHeight = MaxStepHeight;
	AsyncInput.bAlwaysCheckFloor = bAlwaysCheckFloor;
	AsyncInput.WalkableFloorZ = WalkableFloorZ;
	AsyncInput.bUseFlatBaseForFloorChecks = bUseFlatBaseForFloorChecks;
	AsyncInput.GravityZ = GetGravityZ();
	AsyncInput.bCanEverCrouch = CanEverCrouch();
	AsyncInput.MaxSimulationIterations = MaxSimulationIterations;
	AsyncInput.MaxSimulationTimeStep = MaxSimulationTimeStep;
	AsyncInput.bMaintainHorizontalGroundVelocity = bMaintainHorizontalGroundVelocity;
	AsyncInput.bUseSeparateBrakingFriction = bUseSeparateBrakingFriction;
	AsyncInput.GroundFriction = GroundFriction;
	AsyncInput.BrakingFrictionFactor = BrakingFrictionFactor;
	AsyncInput.BrakingFriction = BrakingFriction;
	AsyncInput.BrakingSubStepTime = BrakingSubStepTime;
	AsyncInput.BrakingDecelerationWalking = BrakingDecelerationWalking;
	AsyncInput.BrakingDecelerationFalling = BrakingDecelerationFalling;
	AsyncInput.BrakingDecelerationSwimming = BrakingDecelerationSwimming;
	AsyncInput.BrakingDecelerationFlying = BrakingDecelerationFlying;
	AsyncInput.MaxDepenetrationWithPawn = MaxDepenetrationWithPawn;
	AsyncInput.MaxDepenetrationWithGeometryAsProxy = MaxDepenetrationWithGeometryAsProxy;
	AsyncInput.MaxDepenetrationWithGeometry = MaxDepenetrationWithGeometry;
	AsyncInput.MaxDepenetrationWithPawnAsProxy = MaxDepenetrationWithPawnAsProxy;
	AsyncInput.bCanWalkOffLedgesWhenCrouching = bCanWalkOffLedgesWhenCrouching;
	AsyncInput.bCanWalkOffLedges = bCanWalkOffLedges;
	AsyncInput.LedgeCheckThreshold = LedgeCheckThreshold;
	AsyncInput.PerchRadiusThreshold = PerchRadiusThreshold;
	AsyncInput.AirControl = AirControl;
	AsyncInput.AirControlBoostMultiplier = AirControlBoostMultiplier;
	AsyncInput.AirControlBoostVelocityThreshold = AirControlBoostVelocityThreshold;
	AsyncInput.bDontFallBelowJumpZVelocityDuringJump = bDontFallBelowJumpZVelocityDuringJump;
	AsyncInput.bApplyGravityWhileJumping = bApplyGravityWhileJumping;
	AsyncInput.PhysicsVolumeTerminalVelocity = GetPhysicsVolume()->TerminalVelocity;
	AsyncInput.MaxJumpApexAttemptsPerSimulation = MaxJumpApexAttemptsPerSimulation;
	AsyncInput.DefaultLandMovementMode = DefaultLandMovementMode;
	AsyncInput.FallingLateralFriction = FallingLateralFriction;
	AsyncInput.JumpZVelocity = JumpZVelocity;
	AsyncInput.bAllowPhysicsRotationDuringAnimRootMotion = bAllowPhysicsRotationDuringAnimRootMotion;
	AsyncInput.bDeferUpdateMoveComponent = bDeferUpdateMoveComponent;
	AsyncInput.bRequestedMoveUseAcceleration = bRequestedMoveUseAcceleration;
	AsyncInput.PerchAdditionalHeight = PerchAdditionalHeight;
	AsyncInput.bNavAgentPropsCanJump = NavAgentProps.bCanJump;
	AsyncInput.bMovementStateCanJump = MovementState.bCanJump;
	AsyncInput.MaxWalkSpeedCrouched = MaxWalkSpeedCrouched;
	AsyncInput.MaxWalkSpeed = MaxWalkSpeed;
	AsyncInput.MaxSwimSpeed = MaxSwimSpeed;
	AsyncInput.MaxFlySpeed = MaxFlySpeed;
	AsyncInput.MaxCustomMovementSpeed = MaxCustomMovementSpeed;
	AsyncInput.RotationRate = RotationRate;
	
	AsyncInput.RootMotion = AsyncRootMotion;
	AsyncRootMotion.Clear();

	FCachedMovementBaseAsyncData& MovementBaseData = AsyncInput.MovementBaseAsyncData;
	
	UPrimitiveComponent* MovementBase = Cast<UPrimitiveComponent>(GetMovementBaseObject());
	MovementBaseData.CachedMovementBase = MovementBase;
	MovementBaseData.OldBaseQuat = OldBaseQuat;
	MovementBaseData.OldBaseLocation = OldBaseLocation;
	if (IsValid(MovementBase))
	{
		MovementBaseData.bMovementBaseIsValidCached = true;
		MovementBaseData.bMovementBaseOwnerIsValidCached = true;
		
		// Todo: eventually switch async code to use new FMovementBaseInterfaceData struct properly rather than converting here
		const FMovementBaseInterfaceData MovementBaseInterfaceData(MovementBase);
		MovementBaseData.bMovementBaseUsesRelativeLocationCached = MovementBaseUtility::UseRelativeLocation(&MovementBaseInterfaceData);
		MovementBaseData.bMovementBaseIsSimulatedCached = MovementBaseUtility::IsSimulatedBase(&MovementBaseInterfaceData);
		MovementBaseData.bMovementBaseIsDynamicCached = MovementBaseUtility::IsDynamicBase(&MovementBaseInterfaceData);
		MovementBaseData.bIsBaseTransformValid = MovementBaseUtility::GetMovementBaseTransform(&MovementBaseInterfaceData, CharacterOwner->GetBasedMovement().BoneName, MovementBaseData.BaseLocation, MovementBaseData.BaseQuat);
	}
	else
	{
		MovementBaseData.bMovementBaseIsValidCached = false;
		MovementBaseData.bMovementBaseOwnerIsValidCached = false;
		MovementBaseData.bMovementBaseUsesRelativeLocationCached = false;
		MovementBaseData.bMovementBaseIsSimulatedCached = false;
		MovementBaseData.bMovementBaseIsDynamicCached = false;
		MovementBaseData.BaseLocation = FVector::ZeroVector;
		MovementBaseData.BaseQuat = FQuat::Identity;
		MovementBaseData.bIsBaseTransformValid = false;
	}

	// Character owner inputs
	TUniquePtr<FCharacterAsyncInput>& CharacterInput = AsyncInput.CharacterInput;
	ensure(CharacterInput.IsValid());
	CharacterOwner->FillAsyncInput(*CharacterInput.Get());

	AsyncInput.World = GetWorld();

	if (ensure(UpdatedComponent))
	{
		TUniquePtr<FUpdatedComponentAsyncInput>& UpdatedComponentInput = AsyncInput.UpdatedComponentInput;
		ensure(UpdatedComponentInput.IsValid());

		UpdatedComponentInput->bIsQueryCollisionEnabled = UpdatedComponent->IsQueryCollisionEnabled();
		UpdatedComponentInput->bIsSimulatingPhysics = UpdatedComponent->IsSimulatingPhysics();
		UpdatedComponentInput->PhysicsHandle = nullptr;

		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(UpdatedComponent))
		{
			static const FName TraceTagName = "MoveComponent";
			UpdatedComponentInput->bForceGatherOverlaps = !FUpdatedComponentAsyncInput::ShouldCheckOverlapFlagToQueueOverlaps(*PrimitiveComponent);
			UpdatedComponentInput->bGatherOverlaps = PrimitiveComponent->GetGenerateOverlapEvents() || UpdatedComponentInput->bForceGatherOverlaps;
			UpdatedComponentInput->UpdatedComponent = PrimitiveComponent;
			UpdatedComponentInput->PhysicsHandle = PrimitiveComponent->BodyInstance.GetPhysicsActor();
			ensure(UpdatedComponentInput->PhysicsHandle);

			// TODO Double check we initialize all these SQ params correctly and use them in the right spots.

			// Params for MoveComponent queries.
			UpdatedComponentInput->MoveComponentQueryParams = FComponentQueryParams(SCENE_QUERY_STAT(MoveComponent), GetOwner());
			PrimitiveComponent->InitSweepCollisionParams(UpdatedComponentInput->MoveComponentQueryParams, UpdatedComponentInput->MoveComponentCollisionResponseParams);
			UpdatedComponentInput->MoveComponentQueryParams.bIgnoreTouches |= !(UpdatedComponentInput->bGatherOverlaps);
			UpdatedComponentInput->MoveComponentQueryParams.TraceTag = TraceTagName;
			UpdatedComponentInput->CollisionShape = PrimitiveComponent->GetCollisionShape(MovementComponentCVars::PenetrationOverlapCheckInflation);

			// Params for FindFloor, CharMovement queries.
			AsyncInput.QueryParams = FComponentQueryParams(SCENE_QUERY_STAT(ComputeFloorDist), GetOwner());
			PrimitiveComponent->InitSweepCollisionParams(AsyncInput.QueryParams, AsyncInput.CollisionResponseParams);
			AsyncInput.CollisionChannel = PrimitiveComponent->GetCollisionObjectType();


		}
		else
		{
			ensure(false);
		}

		UpdatedComponentInput->Scale = UpdatedComponent->GetComponentScale();
	}
	AsyncInput.RandomStream.Initialize(FApp::bUseFixedSeed ? GetFName() : NAME_None);


	// Only game thread inputs need to be updated here.
	AsyncInput.GTInputs.bWantsToCrouch = bWantsToCrouch;

	if (MovementMode == MOVE_None)
	{
		// TODO fix this, sometimes we don't fall on start and just float in air because movement mode is none. Not sure why.
		// Pawn typically sets movement mode after possess, but something goes wrong.
		AsyncInput.GTInputs.MovementMode = GroundMovementMode;
		AsyncInput.GTInputs.bValidMovementMode = true;
	}
	else
	{
		AsyncInput.GTInputs.MovementMode = MovementMode;
		AsyncInput.GTInputs.bValidMovementMode = bMovementModeDirty;
	}
	AsyncInput.GTInputs.bPressedJump = CharacterOwner->bPressedJump;


	if(AsyncSimState->IsValid() == false)
	{
		// Need to fully initialize output as we do not have any stored async state.
		AsyncSimState->bWasSimulatingRootMotion = bWasSimulatingRootMotion;
		AsyncSimState->MovementMode = MovementMode;
		AsyncSimState->GroundMovementMode = GroundMovementMode;
		AsyncSimState->CustomMovementMode = CustomMovementMode;
		AsyncSimState->Acceleration = Acceleration;
		AsyncSimState->AnalogInputModifier = AnalogInputModifier;
		AsyncSimState->LastUpdateLocation = LastUpdateLocation;
		AsyncSimState->LastUpdateRotation = LastUpdateRotation;
		AsyncSimState->LastUpdateVelocity = LastUpdateVelocity;
		AsyncSimState->bForceNextFloorCheck = bForceNextFloorCheck;
		AsyncSimState->Velocity = Velocity;
		AsyncSimState->LastPreAdditiveVelocity = FVector::ZeroVector;
		AsyncSimState->bIsAdditiveVelocityApplied = false;
		AsyncSimState->bDeferUpdateBasedMovement = bDeferUpdateBasedMovement;
		AsyncSimState->MoveComponentFlags = MoveComponentFlags;
		AsyncSimState->PendingForceToApply = PendingForceToApply;
		AsyncSimState->PendingImpulseToApply = PendingImpulseToApply;
		AsyncSimState->PendingLaunchVelocity = PendingLaunchVelocity;
		AsyncSimState->bCrouchMaintainsBaseLocation = bCrouchMaintainsBaseLocation;
		AsyncSimState->bJustTeleported = bJustTeleported;
		CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleSize(AsyncSimState->ScaledCapsuleRadius, AsyncSimState->ScaledCapsuleHalfHeight);
		AsyncSimState->bIsCrouched = CharacterOwner->IsCrouched();
		AsyncSimState->bWantsToCrouch = bWantsToCrouch;
		AsyncSimState->bMovementInProgress = bMovementInProgress;
		AsyncSimState->CurrentFloor = CurrentFloor;
		AsyncSimState->bHasRequestedVelocity = bHasRequestedVelocity;
		AsyncSimState->bRequestedMoveWithMaxSpeed = bRequestedMoveWithMaxSpeed;
		AsyncSimState->RequestedVelocity = RequestedVelocity;
		AsyncSimState->LastUpdateRequestedVelocity = LastUpdateRequestedVelocity;
		AsyncSimState->NumJumpApexAttempts = NumJumpApexAttempts;
		AsyncSimState->bShouldApplyDeltaToMeshPhysicsTransforms = false;
		AsyncSimState->DeltaPosition = FVector::ZeroVector;
		AsyncSimState->DeltaQuat = FQuat::Identity;
		AsyncSimState->DeltaTime = 0.0f; // Fill out in async callback.
		AsyncSimState->OldLocation = UpdatedComponent ? UpdatedComponent->GetComponentLocation() : FVector::ZeroVector;
		AsyncSimState->OldVelocity = Velocity;
		AsyncSimState->bShouldDisablePostPhysicsTick = false;
		AsyncSimState->bShouldEnablePostPhysicsTick = false;
		AsyncSimState->bShouldAddMovementBaseTickDependency = false;
		AsyncSimState->bShouldRemoveMovementBaseTickDependency = false;
		AsyncSimState->NewMovementBase = AsyncInput.MovementBaseAsyncData.CachedMovementBase;
		AsyncSimState->NewMovementBaseOwner = AsyncSimState->NewMovementBase ? AsyncSimState->NewMovementBase->GetOwner() : nullptr;

		// Character owner data
		TUniquePtr<FCharacterAsyncOutput>& CharacterOutput = AsyncSimState->CharacterOutput;
		ensure(CharacterOutput.IsValid());
		CharacterOwner->InitializeAsyncOutput(*CharacterOutput.Get());

		AsyncSimState->bIsValid = true;
	}
}

void UCharacterMovementComponent::BuildAsyncInput()
{
	if (CharacterMovementCVars::AsyncCharacterMovement == 1  && IsAsyncCallbackRegistered())
	{
		FCharacterMovementComponentAsyncInput* Input = AsyncCallback->GetProducerInputData_External();
		if (Input->bInitialized == false)
		{
			Input->Initialize<FCharacterMovementComponentAsyncInput::FCharacterInput, FCharacterMovementComponentAsyncInput::FUpdatedComponentInput>();
		}

		if (AsyncSimState.IsValid() == false)
		{
			AsyncSimState = MakeShared<FCharacterMovementComponentAsyncOutput, ESPMode::ThreadSafe>();
		}
		Input->AsyncSimState = AsyncSimState;

		const FVector InputVector = ConsumeInputVector();
		FillAsyncInput(InputVector, *Input);

		PostBuildAsyncInput();
	}
}

void UCharacterMovementComponent::PostBuildAsyncInput()
{
	// Reset so we can tell if movement mode change comes from game thread.
	bMovementModeDirty = false;
	
	// This is slightly sketchy, but bIsValid will only ever be set on game thread so this is threadsafe.
	// TODO make that more clear.
	if (AsyncSimState->IsValid() == false)
	{
		AsyncSimState->bIsValid = true;
	}
}

void UCharacterMovementComponent::ApplyAsyncOutput(FCharacterMovementComponentAsyncOutput& Output)
{
	ensure(Output.DeltaTime > 0.0f);

	if (Output.IsValid() == false)
	{
		return;
	}


	// TODO does anyhting here need to be interpolated?

	// TODO:
	// Not all of this stuff should actually be copied to game thread, if it is not read by other GT
	// systems it doesn't need to be copied back. Some values representing inputs will cause issues if copied back
	// (like bPressedJump for example)
	// Need to sort through and see what should/shouldn't be copied back from outputs.


	// TODO verify order
	bWasSimulatingRootMotion = Output.bWasSimulatingRootMotion;
	Acceleration = Output.Acceleration;
	AnalogInputModifier = Output.AnalogInputModifier;
	LastUpdateLocation = Output.LastUpdateLocation;
	LastUpdateRotation = Output.LastUpdateRotation;
	LastUpdateVelocity = Output.LastUpdateVelocity;
	bForceNextFloorCheck = Output.bForceNextFloorCheck;

	EMovementMode PrevMovementMode = MovementMode;
	uint8 PrevCustomMode = CustomMovementMode;
	MovementMode = Output.MovementMode;
	CustomMovementMode = Output.CustomMovementMode;
	if (CharacterOwner && (MovementMode != PrevMovementMode || CustomMovementMode != PrevCustomMode))
	{
		CharacterOwner->OnMovementModeChanged(PrevMovementMode, PrevCustomMode);
	}

	Velocity = Output.Velocity;
	CallMovementUpdateDelegate(Output.DeltaTime, Output.OldLocation, Output.OldVelocity);

	bDeferUpdateBasedMovement = Output.bDeferUpdateBasedMovement;// TODO verify
	MoveComponentFlags = Output.MoveComponentFlags;

	// Do these need to be copied back?
	PendingForceToApply = Output.PendingForceToApply;
	PendingImpulseToApply = Output.PendingImpulseToApply;
	PendingLaunchVelocity = Output.PendingLaunchVelocity;

	bCrouchMaintainsBaseLocation = Output.bCrouchMaintainsBaseLocation;
	bJustTeleported = Output.bJustTeleported;

	// TODO Crouching, check if scaled radius/half haeight changed on capsule and handle?
	ensure(Output.bIsCrouched == false);
	bWantsToCrouch = Output.bWantsToCrouch;

	bMovementInProgress = Output.bMovementInProgress;
	CurrentFloor = Output.CurrentFloor;
	bHasRequestedVelocity = Output.bHasRequestedVelocity;
	bRequestedMoveWithMaxSpeed = Output.bRequestedMoveWithMaxSpeed;
	RequestedVelocity = Output.RequestedVelocity;
	LastUpdateRequestedVelocity = Output.LastUpdateRequestedVelocity;

	if (CharacterOwner)
	{
		TUniquePtr<FCharacterAsyncOutput>& CharacterOutput = Output.CharacterOutput;
		ensure(CharacterOutput.IsValid());
		CharacterOwner->ApplyAsyncOutput(*CharacterOutput.Get());
	}

	NumJumpApexAttempts = Output.NumJumpApexAttempts;

	if (CharacterOwner && Output.bShouldApplyDeltaToMeshPhysicsTransforms)
	{
		CharacterOwner->GetMesh()->ApplyDeltaToAllPhysicsTransforms(Output.DeltaPosition, Output.DeltaQuat);
	}

	// TODO how to handle tick group changes correctly
	//ensure(Output.bShouldDisablePostPhysicsTick == false);
	//ensure(Output.bShouldEnablePostPhysicsTick == false);
	//ensure(Output.bShouldAddMovementBaseTickDependency == false);
	//ensure(Output.bShouldRemoveMovementBaseTickDependency == false);

	// TODO apply new movement base


	// TODO process overlap events

	// TODO Should this happen before or after movement update above?
	if (CharacterOwner)
	{
		CharacterOwner->FaceRotation(Output.CharacterOutput->Rotation);
	}

	// TODO MovementBase
	// Need to call SavedBaseLocation?
	// Call SetBase with NewMovementBase
}

void UCharacterMovementComponent::ProcessAsyncOutput()
{
	if (CharacterMovementCVars::AsyncCharacterMovement == 1 && IsAsyncCallbackRegistered())
	{
		while (auto Output = AsyncCallback->PopOutputData_External())
		{
			ApplyAsyncOutput(*Output);
		}
	}
}

void UCharacterMovementComponent::RegisterAsyncCallback()
{
	if (CharacterMovementCVars::AsyncCharacterMovement == 1)
	{
		if (UWorld* World = GetWorld())
		{
			if (FPhysScene* PhysScene = World->GetPhysicsScene())
			{
				AsyncCallback = PhysScene->GetSolver()->CreateAndRegisterSimCallbackObject_External<FCharacterMovementComponentAsyncCallback>();
			}
		}
	}
}

bool UCharacterMovementComponent::IsAsyncCallbackRegistered() const
{
	return AsyncCallback != nullptr;
}
