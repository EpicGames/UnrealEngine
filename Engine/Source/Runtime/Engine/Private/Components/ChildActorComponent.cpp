// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChildActorComponent.h"
#include "Blueprint/BlueprintSupport.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "UObject/Package.h"
#include "Net/UnrealNetwork.h"
#include "Engine/Engine.h"
#include "Engine/DemoNetDriver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChildActorComponent)

#if WITH_EDITOR
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "Misc/DataValidation.h"
#endif
#include "Iris/ReplicationSystem/PropertyReplicationFragment.h"

DEFINE_LOG_CATEGORY_STATIC(LogChildActorComponent, Log, All);

static int32 GExperimentalAllowPerInstanceChildActorProperties = 0;

static FAutoConsoleVariableRef CVarExperimentalAllowPerInstanceChildActorProperties(
	TEXT("cac.ExperimentalAllowPerInstanceChildActorProperties"),
	GExperimentalAllowPerInstanceChildActorProperties,
	TEXT("[EXPERIMENTAL] If true, allows properties to be modified on a per-instance basis for child actors."),
	ECVF_Default
);

bool UChildActorComponent::ExperimentalAllowPerInstanceChildActorProperties()
{
	return GExperimentalAllowPerInstanceChildActorProperties != 0;
}

UChildActorComponent::UChildActorComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ActorOuter(nullptr)
	, CachedInstanceData(nullptr)
	, bNeedsRecreate(false)
	, bChildActorNameIsExact(false)
{
	bAllowReregistration = false;
	bChildActorIsTransient = false;
	bSetOwner = false;

#if WITH_EDITORONLY_DATA
	EditorTreeViewVisualizationMode = EChildActorComponentTreeViewVisualizationMode::UseDefault;
#endif
}

void UChildActorComponent::OnRegister()
{
	Super::OnRegister();

	if (ActorOuter)
	{
		if (ActorOuter != GetOwner()->GetOuter())
		{
			ChildActorName = NAME_None;
		}

		ActorOuter = nullptr;
	}

	if (ChildActor)
	{
		if (ChildActor->GetClass() != ChildActorClass)
		{
			bNeedsRecreate = true;
			ChildActorName = NAME_None;
		}
		else
		{
			ChildActorName = ChildActor->GetFName();
		}

		if (bNeedsRecreate)
		{
			bNeedsRecreate = false;

			// Avoid dirtying packages if not necessary
			FName PreviousChildActorName = ChildActorName;
			bool bChildActorPackageWasDirty = ChildActor->GetPackage()->IsDirty();
			bool bPackageWasDirty = GetPackage()->IsDirty();

			UE::Net::FScopedIgnoreStaticActorDestruction ScopedIgnoreDestruction;

			DestroyChildActor();
			CreateChildActor();
			
			if (ChildActor && ChildActorName == PreviousChildActorName)
			{
				ChildActor->GetPackage()->SetDirtyFlag(bChildActorPackageWasDirty);
				GetPackage()->SetDirtyFlag(bPackageWasDirty);
			}
		}
		else
		{
			// Ensure the components replication is correctly initialized
			SetIsReplicated(ChildActor->GetIsReplicated());

			// All other paths register this delegate through CreateChildActor; this is the only path that doesn't go there
			RegisterChildActorDestroyedDelegate();
		}
	}
	else if (ChildActorClass)
	{
		CreateChildActor();
	}
	else
	{
		DeleteCachedInstanceData();
		CachedInstanceData = new FChildActorComponentInstanceData(this);
	}
}

void UChildActorComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.HasAllPortFlags(PPF_DuplicateForPIE))
	{
		// PIE duplication should just work normally
		Ar << ChildActorTemplate;
	}
	else if (Ar.HasAllPortFlags(PPF_Duplicate))
	{
		if (GIsEditor && Ar.IsLoading() && !IsTemplate())
		{
			// If we're not a template then we do not want the duplicate so serialize manually and destroy the template that was created for us
			Ar.Serialize(&ChildActorTemplate, sizeof(UObject*));

			if (AActor* UnwantedDuplicate = static_cast<AActor*>(FindObjectWithOuter(this, AActor::StaticClass())))
			{
				UnwantedDuplicate->MarkAsGarbage();
			}
		}
		else if (!GIsEditor && !Ar.IsLoading() && !GIsDuplicatingClassForReinstancing)
		{
			// Avoid the archiver in the duplicate writer case because we want to avoid the duplicate being created
			Ar.Serialize(&ChildActorTemplate, sizeof(UObject*));
		}
		else
		{
			// When we're loading outside of the editor we won't have created the duplicate, so its fine to just use the normal path
			// When we're loading a template then we want the duplicate, so it is fine to use normal archiver
			// When we're saving in the editor we'll create the duplicate, but on loading decide whether to take it or not
			Ar << ChildActorTemplate;
		}
	}

#if WITH_EDITOR
	if (ChildActorClass == nullptr)
	{
#if DO_CHECK
		if (FBlueprintSupport::IsClassPlaceholder(ChildActorClass.DebugAccessRawClassPtr()))
		{
			ensureMsgf(false, TEXT("Unexpectedly encountered placeholder class while serializing a component"));
		}
		else
#endif
		{
			if (!FBlueprintSupport::IsDeferredDependencyPlaceholder(ChildActorTemplate))
			{
				// If ChildActorClass is null then the ChildActorTemplate needs to be as well
				// In certain cases with inheritance clearing ChildActorClass in a grandparent does
				// not clear the ChildActorTemplate on load, so we clear it here 
				ChildActorTemplate = nullptr;
			}
		}
	}
	// Since we sometimes serialize properties instead of using duplication and we can end up pointing at the wrong template
	else if (!Ar.IsPersistent() && ChildActorTemplate)
	{
		if (IsTemplate())
		{
			// If we are a template and are not pointing at a component we own we'll need to fix that
			if (ChildActorTemplate->GetOuter() != this)
			{
				const FString TemplateName = FString::Printf(TEXT("%s_%s_CAT"), *GetName(), *ChildActorClass->GetName());
				if (UObject* ExistingTemplate = StaticFindObject(nullptr, this, *TemplateName))
				{
					ChildActorTemplate = CastChecked<AActor>(ExistingTemplate);
				}
				else
				{
					ChildActorTemplate = CastChecked<AActor>(StaticDuplicateObject(ChildActorTemplate, this, *TemplateName));
				}
			}
		}
		else
		{
			// Because the template may have fixed itself up, the tagged property delta serialized for 
			// the instance may point at a trashed template, so always repoint us to the archetypes template
			ChildActorTemplate = CastChecked<UChildActorComponent>(GetArchetype())->ChildActorTemplate;
		}
	}
#endif
}

void UChildActorComponent::DeleteCachedInstanceData()
{
	if (CachedInstanceData)
	{
		delete CachedInstanceData;
		CachedInstanceData = nullptr;
	}
}

#if WITH_EDITOR
void UChildActorComponent::SetPackageExternal(bool bExternal, bool bShouldDirty)
{
	Modify(bShouldDirty);
	DestroyChildActor();
	CreateChildActor();
}

void UChildActorComponent::PostEditImport()
{
	Super::PostEditImport();

	if (IsTemplate())
	{
		TArray<UObject*> Children;
		GetObjectsWithOuter(this, Children, EGetObjectsFlags::None);

		for (UObject* Child : Children)
		{
			if (Child->GetClass() == ChildActorClass)
			{
				ChildActorTemplate = CastChecked<AActor>(Child);
				break;
			}
		}
	}
	else
	{
		ChildActorTemplate = CastChecked<UChildActorComponent>(GetArchetype())->ChildActorTemplate;
	}

	// Any cached instance data is invalid if we've had data imported in to us
	DeleteCachedInstanceData();
}

void UChildActorComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UChildActorComponent, ChildActorClass))
	{
		ChildActorName = NAME_None;

		if (IsTemplate())
		{
			// This case is necessary to catch the situation where we are propogating the change down to child blueprints
			SetChildActorClass(ChildActorClass);
		}
		else
		{
			UChildActorComponent* Archetype = CastChecked<UChildActorComponent>(GetArchetype());
			ChildActorTemplate = (Archetype->ChildActorClass == ChildActorClass ? Archetype->ChildActorTemplate : nullptr);
		}

		// If this was created by construction script, the post edit change super call will destroy it anyways
		if (!IsCreatedByConstructionScript())
		{
			DestroyChildActor();
			CreateChildActor();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UChildActorComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UChildActorComponent, ChildActorClass))
	{
		if (IsTemplate())
		{
			SetChildActorClass(ChildActorClass);
		}
		else
		{
			ChildActorTemplate = CastChecked<UChildActorComponent>(GetArchetype())->ChildActorTemplate;
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

void UChildActorComponent::PostEditUndo()
{
	Super::PostEditUndo();

	// This hack exists to fix up known cases where the AttachChildren array is broken in very problematic ways.
	// The correct fix will be to use a Transaction Annotation at the SceneComponent level, however, it is too risky
	// to do right now, so this will go away when that is done.
	for (auto& Component : FDirectAttachChildrenAccessor::Get(this))
	{
		if (Component)
		{
			if (!IsValid(Component) && Component->GetOwner() == ChildActor)
			{
				Component = ChildActor->GetRootComponent();
			}
		}
	}
	
}
#endif

AActor* UChildActorComponent::GetSpawnableChildActorTemplate() const
{
	// Only use the instance if it's the same type as the class it was supposedly built from
	if (ChildActorTemplate && ChildActorTemplate->GetClass() == ChildActorClass)
	{
		return ChildActorTemplate;
	}
	// Use the CDO of the class as the template if the instance is wrong.
	else
	{
		return ChildActorClass ? ChildActorClass->GetDefaultObject<AActor>() : nullptr;
	}
}

void UChildActorComponent::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UChildActorComponent, ChildActor);
}

void FActorParentComponentSetter::Set(AActor* ChildActor, UChildActorComponent* ParentComponent)
{
	ChildActor->ParentComponent = ParentComponent;
}

void UChildActorComponent::PostRepNotifies()
{
	Super::PostRepNotifies();

	if (ChildActor)
	{
		FActorParentComponentSetter::Set(ChildActor, this);

		ChildActorClass = ChildActor->GetClass();
		ChildActorName = ChildActor->GetFName();
	}
	else
	{
		ChildActorClass = nullptr;
		ChildActorName = NAME_None;
	}
}

void UChildActorComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

	DestroyChildActor();
}

void UChildActorComponent::OnUnregister()
{
	Super::OnUnregister();
	ActorOuter = GetOwner()->GetOuter();
	DestroyChildActor();
}

FChildActorComponentInstanceData::FChildActorComponentInstanceData(const UChildActorComponent* Component)
	: FSceneComponentInstanceData(Component)
	, ChildActorClass(Component->GetChildActorClass())
	, ChildActorName(Component->GetChildActorName())
	, ComponentInstanceData(nullptr)
{
	if (AActor* ChildActor = Component->GetChildActor())
	{
#if WITH_EDITOR
		ChildActorGUID = ChildActor->GetActorGuid();
#endif
		if (ChildActorName.IsNone())
		{
			ChildActorName = ChildActor->GetFName();
		}

		if (UChildActorComponent::ExperimentalAllowPerInstanceChildActorProperties())
		{
			ActorInstanceData = MakeShared<FActorInstanceData>(ChildActor);
			// If it is empty dump it
			if (!ActorInstanceData->HasInstanceData())
			{
				ActorInstanceData.Reset();
			}
		}

		ComponentInstanceData = MakeShared<FComponentInstanceDataCache>(ChildActor);
		// If it is empty dump it
		if (!ComponentInstanceData->HasInstanceData())
		{
			ComponentInstanceData.Reset();
		}

		USceneComponent* ChildRootComponent = ChildActor->GetRootComponent();
		if (ChildRootComponent)
		{
			for (USceneComponent* AttachedComponent : ChildRootComponent->GetAttachChildren())
			{
				if (AttachedComponent)
				{
					AActor* AttachedActor = AttachedComponent->GetOwner();
					if (AttachedActor != ChildActor)
					{
						FChildActorAttachedActorInfo Info;
						Info.Actor = AttachedActor;
						Info.SocketName = AttachedComponent->GetAttachSocketName();
						Info.RelativeTransform = AttachedComponent->GetRelativeTransform();
						AttachedActors.Add(Info);
					}
				}
			}
		}
	}
}

bool FChildActorComponentInstanceData::ContainsData() const
{
	return AttachedActors.Num() > 0 || (ComponentInstanceData && ComponentInstanceData->HasInstanceData()) || Super::ContainsData();
}

void FChildActorComponentInstanceData::ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase)
{
	Super::ApplyToComponent(Component, CacheApplyPhase);
	CastChecked<UChildActorComponent>(Component)->ApplyComponentInstanceData(this, CacheApplyPhase);
}

void FChildActorComponentInstanceData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
	if (ComponentInstanceData)
	{
		ComponentInstanceData->AddReferencedObjects(Collector);
	}
}

void UChildActorComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UChildActorComponent* This = CastChecked<UChildActorComponent>(InThis);

	if (This->CachedInstanceData)
	{
		This->CachedInstanceData->AddReferencedObjects(Collector);
	}

	Super::AddReferencedObjects(InThis, Collector);
}

void UChildActorComponent::BeginDestroy()
{
	Super::BeginDestroy();

	DeleteCachedInstanceData();
}

TStructOnScope<FActorComponentInstanceData> UChildActorComponent::GetComponentInstanceData() const
{
	TStructOnScope<FActorComponentInstanceData> InstanceData;
	if (CachedInstanceData)
	{
		InstanceData.InitializeAs<FChildActorComponentInstanceData>(*CachedInstanceData);
		delete CachedInstanceData;
		CachedInstanceData = nullptr;
	}
	else
	{
		InstanceData.InitializeAs<FChildActorComponentInstanceData>(this);
	}
	return InstanceData;
}

void UChildActorComponent::ApplyComponentInstanceData(FChildActorComponentInstanceData* ChildActorInstanceData, const ECacheApplyPhase CacheApplyPhase)
{
	check(ChildActorInstanceData);

	if (ChildActorClass == ChildActorInstanceData->ChildActorClass)
	{
		ChildActorName = ChildActorInstanceData->ChildActorName;
	}

	if (!ChildActor ||
		ChildActor->GetClass() != ChildActorClass)
	{
		CreateChildActor(); 
	}

	if (ChildActor)
	{
		// Only rename if it is safe to, and it is needed
		if(ChildActorName != NAME_None &&
		   ChildActor != nullptr &&
		   ChildActor->GetFName() != ChildActorName)
		{
			const FString ChildActorNameString = ChildActorName.ToString();
			if (ChildActor->Rename(*ChildActorNameString, nullptr, REN_Test | REN_AllowPackageLinkerMismatch))
			{
				ChildActor->Rename(*ChildActorNameString, nullptr, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
#if WITH_EDITOR
				ChildActor->ClearActorLabel();
#endif
			}
		}

		if (ChildActorInstanceData->ActorInstanceData)
		{
			ChildActorInstanceData->ActorInstanceData->ApplyToActor(ChildActor, CacheApplyPhase);
		}

		if (ChildActorInstanceData->ComponentInstanceData)
		{
			ChildActorInstanceData->ComponentInstanceData->ApplyToActor(ChildActor, CacheApplyPhase);
		}

		USceneComponent* ChildActorRoot = ChildActor->GetRootComponent();
		if (ChildActorRoot)
		{
			for (const FChildActorAttachedActorInfo& AttachInfo : ChildActorInstanceData->AttachedActors)
			{
				AActor* AttachedActor = AttachInfo.Actor.Get();
				if (AttachedActor)
				{
					USceneComponent* AttachedRootComponent = AttachedActor->GetRootComponent();
					if (AttachedRootComponent)
					{
						AttachedActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
						AttachedRootComponent->AttachToComponent(ChildActorRoot, FAttachmentTransformRules::KeepWorldTransform, AttachInfo.SocketName);
						AttachedRootComponent->SetRelativeTransform(AttachInfo.RelativeTransform);
						AttachedRootComponent->UpdateComponentToWorld();
					}
				}
			}
		}
	}
}

void UChildActorComponent::SetChildActorClass(TSubclassOf<AActor> Class, AActor* ActorTemplate)
{
	ChildActorClass = Class;
	if (IsTemplate())
	{
		if (ChildActorClass)
		{
			if (ChildActorTemplate == nullptr || ActorTemplate || (ChildActorTemplate->GetClass() != ChildActorClass))
			{
				Modify();

				AActor* NewChildActorTemplate = NewObject<AActor>(GetTransientPackage(), ChildActorClass, NAME_None, RF_ArchetypeObject | RF_Transactional | RF_Public, ActorTemplate);

				if (ChildActorTemplate)
				{
					if (ActorTemplate == nullptr)
					{
						UEngine::FCopyPropertiesForUnrelatedObjectsParams Options;
						Options.bNotifyObjectReplacement = true;
						UEngine::CopyPropertiesForUnrelatedObjects(ChildActorTemplate, NewChildActorTemplate, Options);
					}
					ChildActorTemplate->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}

#if WITH_EDITOR
				NewChildActorTemplate->ClearActorLabel();
#endif

				ChildActorTemplate = NewChildActorTemplate;

				// Record initial object state in case we're in a transaction context.
				ChildActorTemplate->Modify();

				// Now set the actual name and outer to the BPGC.
				const FString TemplateName = FString::Printf(TEXT("%s_%s_CAT"), *GetName(), *ChildActorClass->GetName());

				// If the CAC's package is ever reloaded, there's a chance that any serialized template with the same class might've also been reloaded.
				// In order to avoid any collisions, we'll move the reloaded one to the transient package.
				if (AActor* ExistingChildActorTemplate = FindObject<AActor>(this, *TemplateName, EFindObjectFlags::None))
				{
					ExistingChildActorTemplate->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}

				ChildActorTemplate->Rename(*TemplateName, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			}
		}
		else if (ChildActorTemplate)
		{
			Modify();

			ChildActorTemplate->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			ChildActorTemplate = nullptr;
		}
	}
	else
	{
		// Clear actor template if it no longer matches the set class
		if (ChildActorTemplate && ChildActorTemplate->GetClass() != ChildActorClass)
		{
			ChildActorTemplate = nullptr;
		}

		if (IsRegistered())
		{
			ChildActorName = NAME_None;
			DestroyChildActor();

			// If an actor template was supplied, temporarily set ChildActorTemplate to create the new Actor with ActorTemplate used as the template
			TGuardValue<decltype(ChildActorTemplate)> ChildActorTemplateGuard(ChildActorTemplate, (ActorTemplate ? ActorTemplate : ToRawPtr(ChildActorTemplate)));

			CreateChildActor();
		}
		else if (ActorTemplate)
		{
			UE_LOGF(LogChildActorComponent, Warning, "Call to SetChildActorClass on '%ls' supplied ActorTemplate '%ls', but it will not be used due to the component not being registered.", *GetPathName(), *ActorTemplate->GetPathName());
		}
	}
}

#if WITH_EDITOR
void UChildActorComponent::PostLoad()
{
	Super::PostLoad();

	// For a period of time the parent component property on Actor was not a UPROPERTY so this value was not set
	if (ChildActor)
	{
		// Since the template could have been changed we need to respawn the child actor
		// Don't do this if there is no linker which implies component was created via duplication
		if (ChildActorTemplate && GetLinker())
		{
			bNeedsRecreate = true;
		}
		else
		{
			FActorParentComponentSetter::Set(ChildActor, this);
			ChildActor->SetFlags(RF_TextExportTransient | RF_DuplicateTransient);
		}
	}

}

EDataValidationResult UChildActorComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	// If a CAC is set to the class of the blueprint it is currently on, then there will be a cycle and the data is invalid
	const UClass* const Outer = Cast<UClass>(GetOuter());
	if (Outer && Outer == ChildActorClass)
	{
		Result = EDataValidationResult::Invalid;
		Context.AddError(FText::Format(NSLOCTEXT("ChildActorComponent", "ChildActorCycle", "A Child Actor Component's class cannot be set to its owner! '{0}' is an invalid class choice for '{1}'."), FText::FromString(ChildActorClass->GetName()), FText::FromString(GetPathName())));
	}
	return Result;
}
#endif

bool UChildActorComponent::IsChildActorReplicated() const
{
	AActor* ChildClassCDO = (ChildActorClass ? ChildActorClass->GetDefaultObject<AActor>() : nullptr);
	const bool bChildActorClassReplicated = ChildClassCDO && ChildClassCDO->GetIsReplicated();
	const bool bChildActorTemplateReplicated = ChildActorTemplate && (ChildActorTemplate->GetClass() == ChildActorClass) && ChildActorTemplate->GetIsReplicated();

	return (bChildActorClassReplicated || bChildActorTemplateReplicated);
}

bool UChildActorComponent::IsLevelBeingRemoved() const
{
	if (AActor* MyOwner = GetOwner())
	{
		if (ULevel* MyLevel = MyOwner->GetLevel())
		{
			return MyLevel->bIsBeingRemoved;
		}
	}

	return false;
}

void UChildActorComponent::OnRep_ChildActor()
{
	RegisterChildActorDestroyedDelegate();
}

void UChildActorComponent::RegisterChildActorDestroyedDelegate()
{
	if (ChildActor)
	{
		ChildActor->OnDestroyed.AddUniqueDynamic(this, &ThisClass::OnChildActorDestroyed);
	}
}

void UChildActorComponent::OnChildActorDestroyed(AActor* Actor)
{
	if (GExitPurge)
	{
		return;
	}

	if (Actor && (Actor->HasAuthority() || !IsChildActorReplicated()) && !IsLevelBeingRemoved())
	{
		UWorld* World = Actor->GetWorld();
		// World may be nullptr during shutdown
		if (World != nullptr)
		{
			UClass* ChildClass = Actor->GetClass();

			// We would like to make certain that our name is not going to accidentally get taken from us while we're destroyed
			// so we increment ClassUnique beyond our index to be certain of it.  This is ... a bit hacky.
			if (!GFastPathUniqueNameGeneration)
			{
				UpdateSuffixForNextNewObject(Actor->GetOuter(), ChildClass, [Actor](int32& Index) { Index = FMath::Max(Index, Actor->GetFName().GetNumber()); });
			}

			// If we are getting here due to garbage collection we can't rename, so we'll have to abandon this child actor name and pick up a new one
			if (!IsGarbageCollecting())
			{
				const FString ObjectBaseName = FString::Printf(TEXT("DESTROYED_%s_CHILDACTOR"), *ChildClass->GetName());
				Actor->Rename(*MakeUniqueObjectName(Actor->GetOuter(), ChildClass, *ObjectBaseName).ToString(), nullptr, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
			}
			else
			{
				ChildActorName = NAME_None;
				if (CachedInstanceData)
				{
					CachedInstanceData->ChildActorName = NAME_None;
				}
			}
		}
	}

	// While reinstancing we need the reference to remain valid so that we can
	// overwrite it when references to old instances are updated to references
	// to new instances:
	if (!GIsReinstancing)
	{
		ChildActor = nullptr;
	}
}

void UChildActorComponent::CreateChildActor(TFunction<void(AActor*)> CustomizerFunc)
{
	AActor* MyOwner = GetOwner();

	if (MyOwner && !MyOwner->HasAuthority())
	{
		if (IsChildActorReplicated() && !GIsReconstructingBlueprintInstances)
		{
			// If we belong to an actor that is not authoritative and the child class is replicated then we expect that Actor will be replicated across so don't spawn one
			return;
		}
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_CreateChildActor);

	// Kill spawned actor if we have one
	DestroyChildActor();

	// If we have a class to spawn.
	if (ChildActorClass != nullptr)
	{
		UWorld* World = GetWorld();
		if (World != nullptr)
		{
			// If we're in a replay scrub and this is a startup actor and the class exists, assume the reference will be restored later
			if (World->IsPlayingReplay() && World->GetDemoNetDriver()->IsRestoringStartupActors() && MyOwner && MyOwner->IsNetStartupActor() && ChildActorClass)
			{
				UE_LOGF(LogChildActorComponent, Display, "Not creating child actor due to replay! Comp:%ls ChildActor:%ls", *GetPathName(), *GetPathNameSafe(ChildActor));
				return;
			}

			// Before we spawn let's try and prevent cyclic disaster
			bool bSpawn = true;
			AActor* Actor = MyOwner;
			while (Actor && bSpawn)
			{
				if (Actor->GetClass() == ChildActorClass)
				{
					bSpawn = false;
					UE_LOGF(LogChildActorComponent, Error, "Found cycle in child actor component '%ls'.  Not spawning Actor of class '%ls' to break.", *GetPathName(), *ChildActorClass->GetName());
				}
				Actor = Actor->GetParentActor();
			}

			if (bSpawn)
			{
				FActorSpawnParameters Params;
				Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
				Params.bDeferConstruction = true; // We defer construction so that we set ParentComponent prior to component registration so they appear selected
				Params.bAllowDuringConstructionScript = true;
				Params.OverrideLevel = (MyOwner ? MyOwner->GetLevel() : nullptr);
				Params.Name = ChildActorName;

				if (!bChildActorNameIsExact)
				{
					// Note: Requested will remove of _UAID_ from a name
					Params.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
				}

				Params.OverrideParentComponent = this;

				if (bSetOwner)
				{
					Params.Owner = MyOwner;
				}
				
				if (bChildActorIsTransient || HasAllFlags(RF_Transient) || (MyOwner && MyOwner->HasAllFlags(RF_Transient)))
				{
					// If this component or its owner are transient, set our created actor to transient. 
					Params.ObjectFlags |= RF_Transient;
				}

#if WITH_EDITOR
				Params.bCreateActorPackage = false;
				Params.OverridePackage = (MyOwner && !(Params.ObjectFlags & RF_Transient)) ? MyOwner->GetExternalPackage() : nullptr;
				Params.OverrideActorGuid = CachedInstanceData ? CachedInstanceData->ChildActorGUID : FGuid();
				Params.bHideFromSceneOutliner = EditorTreeViewVisualizationMode == EChildActorComponentTreeViewVisualizationMode::Hidden;
#endif
				if (ChildActorTemplate && ChildActorTemplate->GetClass() == ChildActorClass)
				{
					Params.Template = ChildActorTemplate;
				}
				Params.ObjectFlags |= (RF_TextExportTransient | RF_DuplicateTransient);
				
				// If necessary, the actor will eventually be saved to the transaction buffer,
				// but only after we've finished initializing it manually.
				Params.ObjectFlags &= ~RF_Transactional;

				// Spawn actor of desired class
				ConditionalUpdateComponentToWorld();
				FVector Location = GetComponentLocation();
				FRotator Rotation = GetComponentRotation();
				ChildActor = World->SpawnActor(ChildActorClass, &Location, &Rotation, Params);

				// If spawn was successful, 
				if (ChildActor != nullptr)
				{
					if (IsEditorOnly() || (MyOwner && MyOwner->IsEditorOnly()))
					{
						// If this component or its owner are editor only, set our created actor to editor only. 
						ChildActor->bIsEditorOnlyActor = true;
					}
					else
					{
						RegisterChildActorDestroyedDelegate();
					}

					ChildActorName = ChildActor->GetFName();

					if (CustomizerFunc)
					{
						CustomizerFunc(ChildActor);
					}

					// Parts that we deferred from SpawnActor
					const FComponentInstanceDataCache* ComponentInstanceData = (CachedInstanceData ? CachedInstanceData->ComponentInstanceData.Get() : nullptr);
					ChildActor->FinishSpawning(GetComponentTransform(), false, ComponentInstanceData);

					USceneComponent* ChildRootComponent = ChildActor->GetRootComponent();
					
					// Since child actors are expected to be attached to parent actors, they must have a root component.
					if (!ChildRootComponent)
					{
						ChildRootComponent = NewObject<USceneComponent>(ChildActor, USceneComponent::GetDefaultSceneRootVariableName());
						ChildActor->SetRootComponent(ChildRootComponent);
						ChildRootComponent->RegisterComponent();
					}

					{
						TGuardValue<TEnumAsByte<EComponentMobility::Type>> MobilityGuard(ChildRootComponent->Mobility, Mobility);
						ChildRootComponent->AttachToComponent(this, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
					}

					SetIsReplicated(ChildActor->GetIsReplicated());

					if (CachedInstanceData)
					{
						for (const FChildActorAttachedActorInfo& AttachedActorInfo : CachedInstanceData->AttachedActors)
						{
							AActor* AttachedActor = AttachedActorInfo.Actor.Get();

							if (AttachedActor && AttachedActor->GetAttachParentActor() == nullptr)
							{
								AttachedActor->AttachToActor(ChildActor, FAttachmentTransformRules::KeepWorldTransform, AttachedActorInfo.SocketName);
								AttachedActor->SetActorRelativeTransform(AttachedActorInfo.RelativeTransform);
							}
						}
					}

					// Must be called last: ensures all relevant state is saved to the transaction bufffer
					const bool bIsTransactional = HasAllFlags(RF_Transactional);
					if (bIsTransactional)
					{
						ChildActor->SetFlags(RF_Transactional);
						ChildRootComponent->SetFlags(RF_Transactional);
						ChildActor->Modify();
						ChildRootComponent->Modify();
					}
				}
			}
		}
	}

	DeleteCachedInstanceData();

	if (ChildActor)
	{
		OnChildActorCreatedDelegate.Broadcast(ChildActor);
	}
}

void UChildActorComponent::SetChildActorName(const FName InName)
{
	ChildActorName = InName;
}

void UChildActorComponent::SetChildActorNameIsExact(bool bInExact)
{
	bChildActorNameIsExact = bInExact;
}

void UChildActorComponent::DestroyChildActor()
{
	UWorld* LocalWorld = GetWorld();
	AActor* MyOwner = GetOwner();

	// If we're in a replay scrub and this is a startup actor and the class exists, assume the reference will be restored later
	if (LocalWorld && LocalWorld->IsPlayingReplay() && LocalWorld->GetDemoNetDriver()->IsRestoringStartupActors() && MyOwner && MyOwner->IsNetStartupActor() && ChildActorClass)
	{
		UE_LOGF(LogChildActorComponent, Display, "Not creating destroying actor due to replay! Comp:%ls ChildActor:%ls", *GetPathName(), *GetPathNameSafe(ChildActor));
		return;
	}

	// If we own an Actor, kill it now unless we don't have authority on it, for that we rely on the server
	// If the level is being removed then don't destroy the child actor so re-adding it doesn't
	// need to create a new actor

	if (ChildActor && (ChildActor->HasAuthority() || !IsChildActorReplicated()) && !IsLevelBeingRemoved())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_DestroyChildActor);

		if (!GExitPurge)
		{
			// if still alive, destroy, otherwise just clear the pointer
			const bool bIsChildActorPendingKillOrUnreachable = !IsValidChecked(ChildActor) || ChildActor->IsUnreachable();
			if (!bIsChildActorPendingKillOrUnreachable)
			{
#if WITH_EDITOR
				DeleteCachedInstanceData();
#else
				check(!CachedInstanceData);
#endif
				// If we're already tearing down we won't be needing this
				if (!HasAnyFlags(RF_BeginDestroyed) && !IsUnreachable() && !IsBeingDestroyed() && !(MyOwner && MyOwner->IsActorBeingDestroyed()))
				{
					CachedInstanceData = new FChildActorComponentInstanceData(this);
				}
			}

			UWorld* World = ChildActor->GetWorld();
			// World may be nullptr during shutdown
			if (World != nullptr)
			{
				if (!bIsChildActorPendingKillOrUnreachable)
				{
					World->DestroyActor(ChildActor);
				}
			}
		}

		ChildActor = nullptr;
	}
	else if (!IsValid(ChildActor))
	{
		ChildActor = nullptr;
	}
}

void UChildActorComponent::BeginPlay()
{
	Super::BeginPlay();

	if (ChildActor && !ChildActor->HasActorBegunPlay())
	{
		const AActor* Owner = GetOwner();
		const bool bFromLevelStreaming = Owner ? Owner->IsActorBeginningPlayFromLevelStreaming() : false;
		ChildActor->DispatchBeginPlay(bFromLevelStreaming);
	}
}

bool UChildActorComponent::IsHLODRelevant() const
{
	if (!CanBeHLODRelevant(this))
	{
		return false;
	}

	const bool bIsHLODRelevant = ChildActor && ChildActor->IsHLODRelevant();
	return bIsHLODRelevant;
}

void UChildActorComponent::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	using namespace UE::Net;

	// Fallback on default since archetype used on client and server differs giving different default values
	Super::RegisterReplicationFragments(Context, RegistrationFlags | UE::Net::EFragmentRegistrationFlags::InitializeDefaultStateFromClassDefaults);
}


#if WITH_EDITOR
TSubclassOf<class UHLODBuilder> UChildActorComponent::GetCustomHLODBuilderClass() const
{
	// ChildActorComponents are only HLOD relevant so that their child actors are included in the HLOD generation.
	// They don't provide any mesh/visual input, so we route them to be processed by the NullHLODBuilder which 
	// ignores all components sent to it.
	return UNullHLODBuilder::StaticClass();
}

void UChildActorComponent::SetEditorTreeViewVisualizationMode(EChildActorComponentTreeViewVisualizationMode InMode)
{
	Modify();

	EditorTreeViewVisualizationMode = InMode;
}
#endif
