// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ComponentInterfaces.h"
#include "Components/ComponentInterfaceIterator.h"
#include "HAL/IConsoleManager.h"
#include "UnrealEngine.h"


template<class T> 
struct TComponentInterfaceBaseData
{
	static inline TArray<TComponentInterfaceImplementation<T>> Implementers;
	static inline TArray<TComponentInterfaceProvider<T>> Providers;
};

template<class T>
void TComponentInterfaceBase<T>::AddImplementer(const TComponentInterfaceImplementation<T>& Implementer)
{	
	TComponentInterfaceBaseData<T>::Implementers.Add(Implementer);
}

template<class T>
void TComponentInterfaceBase<T>::RemoveImplementer(const UClass* ImplementerClass)
{	
	for (typename TArray<TComponentInterfaceImplementation<T>>::TIterator ImplementerIt(TComponentInterfaceBaseData<T>::Implementers); ImplementerIt; ++ImplementerIt)
	{
		if (ImplementerIt->Class == ImplementerClass)
		{
			ImplementerIt.RemoveCurrent();
		}
	}
}

template<class T>
TArray<TComponentInterfaceImplementation<T>>& TComponentInterfaceBase<T>::GetImplementers()
{
	return TComponentInterfaceBaseData<T>::Implementers;
}

template<class T>
void TComponentInterfaceBase<T>::AddProvider(const TComponentInterfaceProvider<T>& Provider)
{
	TComponentInterfaceBaseData<T>::Providers.Add(Provider);
}

template<class T>
void TComponentInterfaceBase<T>::RemoveProvider(const UClass* ProviderClass)
{
	for (typename TArray<TComponentInterfaceProvider<T>>::TIterator ProviderIt(TComponentInterfaceBaseData<T>::Providers); ProviderIt; ++ProviderIt)
	{
		if (ProviderIt->Class == ProviderClass)
		{
			ProviderIt.RemoveCurrent();
		}
	}
}

template<class T> 
T* TComponentInterfaceProvider<T>::Provides(UObject* SourceObject)
{
	T* Result = nullptr;
	Provides(SourceObject, [&Result](T* Interface) { Result = Interface; return EIteration::Break; });
	
	return Result;
}

template<class T> 
void TComponentInterfaceProvider<T>::Provides(UObject* SourceObject, TArray<T*>& OutResults)
{
	Provides(SourceObject, [&OutResults](T* Interface) { OutResults.Add(Interface); return EIteration::Continue; });
}

template<class T> 
void TComponentInterfaceProvider<T>::Provides(UObject* SourceObject, TFunctionRef<EIteration(T*)> ProvidesFn)
{
	if(!SourceObject)
	{
		return;
	}

	// This could be accelerated by caching all classes in a TMap<> of derived classes of the Provider class to the FComponentInterfaceProvider* 
	for (TComponentInterfaceProvider<T>& Provider : TComponentInterfaceBaseData<T>::Providers)
	{
		if (Provider.Class)
		{
			if (SourceObject->IsA(Provider.Class))
			{
				if (Provider.Provider)
				{
					// If we want to specialization of the provider by derived classes ensure providers are sorted with derived classes first 
					if (T* Interface = Provider.Provider(SourceObject))
					{
						if (ProvidesFn(Interface) == EIteration::Continue)
						{
						}
						else
						{
							return;
						}
					}
				}
				else if (Provider.CollectionProvider)
				{
					TArray<T*> InterfaceCollection = Provider.CollectionProvider(SourceObject);
				
					for (T* Interface : InterfaceCollection)
					{
						if (Interface)
						{
							if (ProvidesFn(Interface) == EIteration::Continue)
							{
							}
							else
							{
								return;
							}
						}
					}
				}
			}
		}
	}
	
	// We also consider interface implementers as providers
	for (TComponentInterfaceImplementation<T>& Implementation : TComponentInterfaceBaseData<T>::Implementers)
	{
		// Collection resolvers potentially provide > 1 object, and need a different interface
		if (Implementation.Class)
		{
			if (SourceObject->IsA(Implementation.Class))
			{
				if (Implementation.Resolver)
				{
					if (T* Interface = Implementation.Resolver(SourceObject))
					{
						if (ProvidesFn(Interface) == EIteration::Continue)
						{
						}
						else
						{
							return;
						}
					}
				}
				else if (Implementation.CollectionResolver)
				{
					TArray<T*> InterfaceCollection = Implementation.CollectionResolver(SourceObject);
				
					for (T* Interface : InterfaceCollection)
					{
						if (Interface)
						{
							if (ProvidesFn(Interface) == EIteration::Continue)
							{
							}
							else
							{
								return;
							}
						}
					}
				}
			}
		}
	}	
}

#define UE_COMPONENT_INTERFACE_INSTANTIATION(Interface)\
	template struct TComponentInterfaceBaseData<Interface>;\
	template class TComponentInterfaceBase<Interface>;\
	template struct TComponentInterfaceProvider<Interface>;

UE_COMPONENT_INTERFACE_INSTANTIATION(IPrimitiveComponent)
UE_COMPONENT_INTERFACE_INSTANTIATION(IStaticMeshComponent)
UE_COMPONENT_INTERFACE_INSTANTIATION(ISkinnedMeshComponent)