// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DecalComponent.cpp: Decal component implementation.
=============================================================================*/

#include "Components/DecalComponent.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "SceneInterface.h"
#include "TimerManager.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include "SceneView.h"
#include "LocalVertexFactory.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MarkActorRenderStateDirtyTask.h"
#include "DeferredDecalSceneProxyDesc.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DecalComponent)

#define LOCTEXT_NAMESPACE "DecalComponent"

static TAutoConsoleVariable<float> CVarDecalFadeDurationScale(
	TEXT("r.Decal.FadeDurationScale"),
	1.0f,
	TEXT("Scales the per decal fade durations. Lower values shortens lifetime and fade duration. Default is 1.0f.")
	);

FDeferredDecalProxy::FDeferredDecalProxy(const UDecalComponent* InComponent)
	: DrawInGame(InComponent->GetVisibleFlag() && !InComponent->bHiddenInGame)
	, DrawInEditor(InComponent->GetVisibleFlag())
	, InvFadeDuration(-1.0f)
	, InvFadeInDuration(1.0f)
	, FadeStartDelayNormalized(1.0f)
	, FadeInStartDelayNormalized(0.0f)
	, FadeScreenSize(InComponent->FadeScreenSize)
	, DecalColor(InComponent->DecalColor)
{
	UMaterialInterface* EffectiveMaterial = UMaterial::GetDefaultMaterial(MD_DeferredDecal);
	UMaterialInterface* ComponentMaterial = InComponent->GetDecalMaterial();

	if (ComponentMaterial)
	{
		UMaterial* BaseMaterial = ComponentMaterial->GetMaterial();

		if (BaseMaterial->MaterialDomain == MD_DeferredDecal)
		{
			EffectiveMaterial = ComponentMaterial;
		}
	}

	Component = InComponent;
	DecalMaterial = EffectiveMaterial;
	SetTransformIncludingDecalSize(InComponent->GetTransformIncludingDecalSize(), InComponent->CalcBounds(InComponent->GetComponentTransform()));
	SortOrder = InComponent->SortOrder;

#if WITH_EDITOR
	// We don't want to fade when we're editing, only in Simulate/PIE/Game
	if (!GIsEditor || (InComponent->GetWorld() && InComponent->GetWorld()->IsPlayInEditor()))
#endif
	{
		InitializeFadingParameters(InComponent->GetWorld()->GetTimeSeconds(), InComponent->GetFadeDuration(), InComponent->GetFadeStartDelay(), InComponent->GetFadeInDuration(), InComponent->GetFadeInStartDelay());
	}
	
	if ( InComponent->GetOwner() )
	{
		DrawInGame &= !(InComponent->GetOwner()->IsHidden());
#if WITH_EDITOR
		DrawInEditor &= !InComponent->GetOwner()->IsHiddenEd();
#endif
	}
}

FDeferredDecalProxy::FDeferredDecalProxy(const USceneComponent* InComponent, UMaterialInterface* InMaterial)
	: DrawInGame(InComponent->GetVisibleFlag() && !InComponent->bHiddenInGame)
	, DrawInEditor(InComponent->GetVisibleFlag())
	, InvFadeDuration(-1.0f)
	, InvFadeInDuration(1.0f)
	, FadeStartDelayNormalized(1.0f)
	, FadeInStartDelayNormalized(0.0f)
	, FadeScreenSize(0.1f)
{
	Component = InComponent;
	DecalMaterial = InMaterial;
	if (InMaterial == nullptr || (InMaterial->GetMaterial()->MaterialDomain != MD_DeferredDecal))
	{
		DecalMaterial = UMaterial::GetDefaultMaterial(MD_DeferredDecal);
	}

	SetTransformIncludingDecalSize(FTransform::Identity, InComponent->CalcBounds(InComponent->GetComponentTransform()));
	SortOrder = 0;

#if WITH_EDITOR
	// We don't want to fade when we're editing, only in Simulate/PIE/Game
	if (!GIsEditor || (InComponent->GetWorld() && InComponent->GetWorld()->IsPlayInEditor()))
#endif
	{
		InitializeFadingParameters(InComponent->GetWorld()->GetTimeSeconds(), 1.0f, 1.0f, 0.0f, 0.0f);
	}

	if (InComponent->GetOwner())
	{
		DrawInGame &= !(InComponent->GetOwner()->IsHidden());
#if WITH_EDITOR
		DrawInEditor &= !InComponent->GetOwner()->IsHiddenEd();
#endif
	}
}

FDeferredDecalProxy::FDeferredDecalProxy(const FDeferredDecalSceneProxyDesc& Desc)
	: DrawInGame(Desc.bDrawInGame)
	, DrawInEditor(Desc.bDrawInEditor)
	, InvFadeDuration(-1.0f)
	, InvFadeInDuration(1.0f)
	, FadeStartDelayNormalized(1.0f)
	, FadeInStartDelayNormalized(0.0f)
	, FadeScreenSize(Desc.FadeScreenSize)
	, DecalColor(Desc.DecalColor)
{
	Component = Desc.Component;
	DecalMaterial = Desc.DecalMaterial;
	SetTransformIncludingDecalSize(Desc.TransformWithDecalScale, Desc.Bounds);
	SortOrder = Desc.SortOrder;

	// We don't want to fade when we're editing, only in Simulate/PIE/Game
	if (Desc.bShouldFade)
	{
		InitializeFadingParameters(Desc.InitializationWorldTimeSeconds, Desc.FadeDuration, Desc.FadeStartDelay, Desc.FadeInDuration, Desc.FadeInStartDelay);
	}
}

void FDeferredDecalProxy::SetTransformIncludingDecalSize(const FTransform& InComponentToWorldIncludingDecalSize, const FBoxSphereBounds& InBounds)
{
	ComponentTrans = InComponentToWorldIncludingDecalSize;
	Bounds = InBounds;
}

void FDeferredDecalProxy::InitializeFadingParameters(float AbsSpawnTime, float FadeDuration, float FadeStartDelay, float FadeInDuration, float FadeInStartDelay)
{
	if (FadeDuration > 0.0f)
	{
		InvFadeDuration = 1.0f / FadeDuration;
		FadeStartDelayNormalized = (AbsSpawnTime + FadeStartDelay + FadeDuration) * InvFadeDuration;
	}
	if(FadeInDuration > 0.0f)
	{
		InvFadeInDuration = 1.0f / FadeInDuration;
		FadeInStartDelayNormalized = (AbsSpawnTime + FadeInStartDelay) * -InvFadeInDuration;
	}
}

FDeferredDecalSceneProxyDesc::FDeferredDecalSceneProxyDesc(const UDecalComponent* InComponent)
{
	Component = InComponent;
	Bounds = InComponent->CalcBounds(InComponent->GetComponentTransform());
	TransformWithDecalScale = InComponent->GetTransformIncludingDecalSize();
	DecalColor = InComponent->DecalColor;
	FadeScreenSize = InComponent->FadeScreenSize;
	FadeDuration = InComponent->GetFadeDuration();
	FadeStartDelay = InComponent->GetFadeStartDelay();
	FadeInDuration = InComponent->GetFadeInDuration();
	FadeInStartDelay = InComponent->GetFadeInStartDelay();
	SortOrder = InComponent->SortOrder;
	bDrawInGame = InComponent->GetVisibleFlag() && !InComponent->bHiddenInGame;
	bDrawInEditor = InComponent->GetVisibleFlag();

	DecalMaterial = [InComponent]
	{
		if(UMaterialInterface* ComponentMaterial = InComponent->GetDecalMaterial())
		{
			const UMaterial* BaseMaterial = ComponentMaterial->GetMaterial();
			if (BaseMaterial->MaterialDomain == MD_DeferredDecal)
			{
				return ComponentMaterial;
			}
		}

		return Cast<UMaterialInterface>(UMaterial::GetDefaultMaterial(MD_DeferredDecal));
	}();

#if WITH_EDITOR
	// We don't want to fade when we're editing, only in Simulate/PIE/Game
	bShouldFade = (!GIsEditor || (InComponent->GetWorld() && InComponent->GetWorld()->IsPlayInEditor()));
#endif

	if (InComponent->GetOwner())
	{
		bDrawInGame &= !(InComponent->GetOwner()->IsHidden());
#if WITH_EDITOR
		bDrawInEditor &= !InComponent->GetOwner()->IsHiddenEd();
#endif
	}
}

bool FDeferredDecalProxy::IsShown( const FSceneView* View ) const
{
	// Logic here should match FPrimitiveSceneProxy::IsShown for consistent behavior in editor and at runtime.
#if WITH_EDITOR
	if ( View->Family->EngineShowFlags.Editor )
	{
		if ( !DrawInEditor )
		{
			return false;
		}
	}
	else
#endif
	{
		if ( !DrawInGame
#if WITH_EDITOR
			|| ( !View->bIsGameView && View->Family->EngineShowFlags.Game && !DrawInEditor )
#endif
			)
		{
			return false;
		}
	}
	return true;
}

UDecalComponent::UDecalComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FadeScreenSize(0.01f)
	, FadeStartDelay(0.0f)
	, FadeDuration(0.0f)
	, bDestroyOwnerAfterFade(true)
	, DecalSize(128.0f, 256.0f, 256.0f)
{
}

void UDecalComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.UEVer() < VER_UE4_DECAL_SIZE)
	{
		DecalSize = FVector(1.0f, 1.0f, 1.0f);
	}
}

bool UDecalComponent::IsPostLoadThreadSafe() const
{
	return true;
}

#if WITH_EDITOR
bool UDecalComponent::GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty)
{
	if(ElementIndex == 0)
	{
		OutOwner = this;
		OutPropertyPath = GET_MEMBER_NAME_STRING_CHECKED(UDecalComponent, DecalMaterial);
		OutProperty = UDecalComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDecalComponent, DecalMaterial));
		return true;
	}

	return false;
}
#endif // WITH_EDITOR

void UDecalComponent::SetLifeSpan(const float LifeSpan)
{
	if (LifeSpan > 0.f)
	{
		GetWorld()->GetTimerManager().SetTimer(TimerHandle_DestroyDecalComponent, this, &UDecalComponent::LifeSpanCallback, LifeSpan, false);
	}
	else
	{
		GetWorld()->GetTimerManager().ClearTimer(TimerHandle_DestroyDecalComponent);
	}
}

void UDecalComponent::LifeSpanCallback()
{
	DestroyComponent();

	if (bDestroyOwnerAfterFade  && (FadeDuration > 0.0f || FadeStartDelay > 0.0f))
	{
		if (AActor* Owner = GetOwner())
		{
			Owner->Destroy();
		}
	}
}

float UDecalComponent::GetFadeStartDelay() const
{
	return FadeStartDelay;
}

float UDecalComponent::GetFadeDuration() const
{
	return FadeDuration;
}

float UDecalComponent::GetFadeInDuration() const
{
	return FadeInDuration;
}

float UDecalComponent::GetFadeInStartDelay() const
{
	return FadeInStartDelay;
}

void UDecalComponent::SetFadeOut(float StartDelay, float Duration, bool DestroyOwnerAfterFade /*= true*/)
{
	float FadeDurationScale = CVarDecalFadeDurationScale.GetValueOnGameThread();
	FadeDurationScale = (FadeDurationScale <= UE_SMALL_NUMBER) ? 0.0f : FadeDurationScale;

	FadeStartDelay = StartDelay * FadeDurationScale;
	FadeDuration = Duration * FadeDurationScale;
	bDestroyOwnerAfterFade = DestroyOwnerAfterFade;

	SetLifeSpan(FadeStartDelay + FadeDuration);

	if(SceneProxy != nullptr)
	{
		GetWorld()->Scene->UpdateDecalFadeOutTime(this);
	}
	else
	{
		MarkRenderStateDirty();
	}
}

void UDecalComponent::SetFadeIn(float StartDelay, float Duration)
{
	FadeInStartDelay = StartDelay;
	FadeInDuration = Duration;

	if (SceneProxy != nullptr)
	{
		GetWorld()->Scene->UpdateDecalFadeInTime(this);
	}
	else
	{
		MarkRenderStateDirty();
	}
}

void UDecalComponent::SetFadeScreenSize(float NewFadeScreenSize)
{
	FadeScreenSize = NewFadeScreenSize;

	MarkRenderStateDirty();
}


void UDecalComponent::SetSortOrder(int32 Value)
{
	SortOrder = Value;

	MarkRenderStateDirty();
}

void UDecalComponent::SetDecalColor(const FLinearColor& InColor)
{
	DecalColor = InColor;

	MarkRenderStateDirty();
}

void UDecalComponent::SetDecalMaterial(class UMaterialInterface* NewDecalMaterial)
{
	DecalMaterial = NewDecalMaterial;

	PrecachePSOs();

	MarkRenderStateDirty();	
}

void UDecalComponent::PostLoad()
{
	Super::PostLoad();

	PrecachePSOs();
}

void UDecalComponent::PrecachePSOs()
{
#if UE_WITH_PSO_PRECACHING
	if (!FApp::CanEverRender() || !IsComponentPSOPrecachingEnabled())
	{
		return;
	}

	if (DecalMaterial && !DecalMaterial->HasAnyFlags(RF_NeedPostLoad))
	{
		PSOPrecacheComponentData.Reset();

		FPSOPrecacheParams PSOPrecacheParams;		
		FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;		
		VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));

		// Immediately create at high priority and thus doesn't need boosting anymore
		TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
		FGraphEventArray PSOPrecacheCompileEvents = DecalMaterial->PrecachePSOs(VertexFactoryDataList, PSOPrecacheParams, EPSOPrecachePriority::High, MaterialPSOPrecacheRequestIDs);
		
		// Request recreate of the render state when the PSO compilation is ready (if we want to delay proxy creation)
		auto Callback = [this]()
		{
			MarkRenderStateDirty();
		};
		PSOPrecacheComponentData.SetMaterialPSOPrecacheData(this, MaterialPSOPrecacheRequestIDs, PSOPrecacheCompileEvents, Callback);
	}
#endif
}

void UDecalComponent::PushSelectionToProxy()
{
	// The decal's proxy does not actually need to know if the decal is selected or not, so there is nothing to do here.
	// This function has been marked as deprecated and can eventually be removed.
}

class UMaterialInterface* UDecalComponent::GetDecalMaterial() const
{
	return DecalMaterial;
}

class UMaterialInstanceDynamic* UDecalComponent::CreateDynamicMaterialInstance()
{
	UMaterialInterface* CurrentMaterial = DecalMaterial;
	
	// If we already set a MID, then we need to create based on its parent.
	if (UMaterialInstanceDynamic* CurrentMaterialMID = Cast<UMaterialInstanceDynamic>(CurrentMaterial))
	{
		CurrentMaterial = CurrentMaterialMID->Parent;
	}

	// Create the MID
	UMaterialInstanceDynamic* NewMaterialInstance = UMaterialInstanceDynamic::Create(CurrentMaterial, this);

	// Assign the MID
	SetDecalMaterial(NewMaterialInstance);

	return NewMaterialInstance;
}

void UDecalComponent::GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials ) const
{
	OutMaterials.Add( GetDecalMaterial() );
}


FDeferredDecalProxy* UDecalComponent::CreateSceneProxy()
{
	LLM_SCOPE(ELLMTag::SceneRender);

#if UE_WITH_PSO_PRECACHING
	if (PSOPrecacheComponentData.IsPSOPrecaching() && GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate)
	{
		return nullptr;
	}
#endif // UE_WITH_PSO_PRECACHING

	return new FDeferredDecalProxy(this);
}

FBoxSphereBounds UDecalComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FVector(0, 0, 0), DecalSize, DecalSize.Size()).TransformBy(LocalToWorld);
}

void UDecalComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITOR
	if (DecalMaterial && DecalMaterial->GetMaterial()->MaterialDomain != MD_DeferredDecal && GEditor)
	{
		static TWeakPtr<class SNotificationItem> NotificationHandle;
		if (!NotificationHandle.IsValid())
		{
			FNotificationInfo Info(LOCTEXT("DecalMaterial_Notify", "Decal Material must use Deferred Decal Material Domain."));
			Info.bFireAndForget = true;
			Info.ExpireDuration = 8.0f;
			Info.SubText = FText::Format(
				LOCTEXT("DecalMaterial_NotifySubtext", "Decal materials must use the Deferred Decal Material Domain.\nEither select a valid material for {0} or open the current material and select the Deferred Decal Material Domain."), 
				FText::FromString(GetOwner()->GetActorNameOrLabel()));
			Info.HyperlinkText = FText::Format(
				LOCTEXT("DecalMaterial_Hyperlink", "Open {0}"), 
				FText::FromString(DecalMaterial->GetName()));
			Info.Hyperlink = FSimpleDelegate::CreateWeakLambda(this, [this]
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(DecalMaterial);
				});

			NotificationHandle = FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
#endif
}

void UDecalComponent::BeginPlay()
{
	Super::BeginPlay();

	SetLifeSpan(FadeStartDelay + FadeDuration);
}

void UDecalComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	// Mimics UPrimitiveComponent's visibility logic, although without the UPrimitiveCompoent visibility flags
	if ( ShouldComponentAddToScene() && ShouldRender() )
	{
		GetWorld()->Scene->AddDecal(this);
	}
}

void UDecalComponent::SendRenderTransform_Concurrent()
{	
	//If Decal isn't hidden update its transform.
	if ( ShouldComponentAddToScene() && ShouldRender() )
	{
		GetWorld()->Scene->UpdateDecalTransform(this);
	}

	Super::SendRenderTransform_Concurrent();
}

const UObject* UDecalComponent::AdditionalStatObject() const
{
	return DecalMaterial;
}

void UDecalComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();
	GetWorld()->Scene->RemoveDecal(this);
}

#if WITH_EDITOR

void UDecalComponent::CheckForErrors()
{
	Super::CheckForErrors();

	if (DecalMaterial && DecalMaterial->GetMaterial()->MaterialDomain != MD_DeferredDecal)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ComponentName"), FText::FromString(GetName()));
		Arguments.Add(TEXT("OwnerName"), FText::FromString(GetNameSafe(GetOwner())));

		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("DecalMaterial_MapCheck", "{ComponentName}::{OwnerName} has a DecalMaterial that doesn't use the Deferred Decal Material Domain."), Arguments)));
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
