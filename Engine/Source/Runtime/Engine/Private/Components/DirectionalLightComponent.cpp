// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DirectionalLightComponent.cpp: DirectionalLightComponent implementation.
=============================================================================*/

#include "Components/DirectionalLightComponent.h"
#include "LightSceneProxy.h"
#include "Math/InverseRotationMatrix.h"
#include "UObject/ConstructorHelpers.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "Engine/Texture2D.h"
#include "SceneManagement.h"
#include "Engine/DirectionalLight.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DirectionalLightComponent)

static float GMaxCSMRadiusToAllowPerObjectShadows = 8000;
static FAutoConsoleVariableRef CVarMaxCSMRadiusToAllowPerObjectShadows(
	TEXT("r.MaxCSMRadiusToAllowPerObjectShadows"),
	GMaxCSMRadiusToAllowPerObjectShadows,
	TEXT("Only stationary lights with a CSM radius smaller than this will create per object shadows for dynamic objects.")
	);

static TAutoConsoleVariable<float> CVarUnbuiltWholeSceneDynamicShadowRadius(
	TEXT("r.Shadow.UnbuiltWholeSceneDynamicShadowRadius"),
	200000.0f,
	TEXT("WholeSceneDynamicShadowRadius to use when using CSM to preview unbuilt lighting from a directional light")
	);

static TAutoConsoleVariable<int32> CVarUnbuiltNumWholeSceneDynamicShadowCascades(
	TEXT("r.Shadow.UnbuiltNumWholeSceneDynamicShadowCascades"),
	4,
	TEXT("DynamicShadowCascades to use when using CSM to preview unbuilt lighting from a directional light"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarCSMShadowDistanceFadeoutMultiplier(
	TEXT("r.Shadow.CSMShadowDistanceFadeoutMultiplier"),
	1.0f,
	TEXT("Multiplier for the CSM distance fade"),
	ECVF_RenderThreadSafe | ECVF_Scalability );

static TAutoConsoleVariable<float> CVarPerObjectCastDistanceRadiusScale(
	TEXT("r.Shadow.PerObjectCastDistanceRadiusScale"),
	8.0f,
	TEXT("PerObjectCastDistanceRadiusScale The scale factor multiplied with the radius of the object to calculate the maximum distance a per-object directional shadow can reach. This will only take effect after a certain (large) radius. Default is 8 times the object radius."),
	ECVF_RenderThreadSafe
	);
static TAutoConsoleVariable<float> CVarPerObjectCastDistanceMin(
	TEXT("r.Shadow.PerObjectCastDistanceMin"),
	(float)UE_FLOAT_HUGE_DISTANCE / 32.0f,
	TEXT("Minimum cast distance for Per-Object shadows, i.e., CastDistDance = Max(r.Shadow.PerObjectCastDistanceRadiusScale * object-radius, r.Shadow.PerObjectCastDistanceMin).\n")
	TEXT("  Default: UE_FLOAT_HUGE_DISTANCE / 32.0f"),
	ECVF_RenderThreadSafe
	);
static TAutoConsoleVariable<int32> CVarMaxNumFarShadowCascades(
	TEXT("r.Shadow.MaxNumFarShadowCascades"),
	10,
	TEXT("Max number of far shadow cascades that can be cast from a directional light"),
	ECVF_RenderThreadSafe | ECVF_Scalability );

ENGINE_API int32 GFarShadowStaticMeshLODBias = 0;
FAutoConsoleVariableRef CVarFarShadowStaticMeshLODBias(
	TEXT("r.Shadow.FarShadowStaticMeshLODBias"),
	GFarShadowStaticMeshLODBias,
	TEXT("Notice: only selected geometry types (static meshes and landscapes) respect this value."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarFarShadowDistanceOverride(
	TEXT("r.Shadow.FarShadowDistanceOverride"),
	0.0f,
	TEXT("Overriding far shadow distance for all directional lighst"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarRtdfFarTransitionScale(
	TEXT("r.DFFarTransitionScale"),
	1.0f,
	TEXT("Use to modify the length of the far transition (fade out) of the distance field shadows.\n")
	TEXT("1.0: (default) Calculate in the same way as other cascades.")
	TEXT("0.0: Disable fade out."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRTDFDistanceScale(
	TEXT("r.DFDistanceScale"),
	1.0f,
	TEXT("Factor to scale directional light property 'DistanceField Shadows Distance', clamped to [0.0001, 10000].\n")
	TEXT("I.e.: DistanceFieldShadowsDistance *= r.DFDistanceScale.\n")	
	TEXT("[0.0001,1): shorter distance\n")
	TEXT(" 1: normal (default)\n")
	TEXT("(1,10000]: larger distance.)"),
	ECVF_RenderThreadSafe);

// Mirror of renderer function, which is inaccessible to the engine component
static bool IsSceneCaptureCubeShareCascadesEnabled(EShaderPlatform Platform)
{
	static const IConsoleVariable* CVarSceneCaptureCubeMobileShareCascades = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SceneCapture.CubeMobileShareCascades"));
	return CVarSceneCaptureCubeMobileShareCascades && CVarSceneCaptureCubeMobileShareCascades->GetBool() && IsMobilePlatform(Platform);
}

// Computes a shadow culling volume (convex hull) based on a set of 8 vertices and a light direction
void ComputeShadowCullingVolume(bool bReverseCulling, const FVector* CascadeFrustumVerts, const FVector& LightDirection, FConvexVolume& ConvexVolumeOut, FPlane& NearPlaneOut, FPlane& FarPlaneOut) 
{
	// Pairs of plane indices from SubFrustumPlanes whose intersections
	// form the edges of the frustum.
	static const int32 AdjacentPlanePairs[12][2] =
	{
		{0,2}, {0,4}, {0,1}, {0,3},
		{2,3}, {4,2}, {1,4}, {3,1},
		{2,5}, {4,5}, {1,5}, {3,5}
	};
	// Maps a plane pair index to the index of the two frustum corners
	// which form the end points of the plane intersection.
	static const int32 LineVertexIndices[12][2] =
	{
		{0,1}, {1,3}, {3,2}, {2,0},
		{0,4}, {1,5}, {3,7}, {2,6},
		{4,5}, {5,7}, {7,6}, {6,4}
	};

	TArray<FPlane, TInlineAllocator<6>> Planes;

	// Find the view frustum subsection planes which face away from the light and add them to the bounding volume
	FPlane SubFrustumPlanes[6];
	if (!bReverseCulling)
	{
		SubFrustumPlanes[0] = FPlane(CascadeFrustumVerts[3], CascadeFrustumVerts[2], CascadeFrustumVerts[0]); // Near
		SubFrustumPlanes[1] = FPlane(CascadeFrustumVerts[7], CascadeFrustumVerts[6], CascadeFrustumVerts[2]); // Left
		SubFrustumPlanes[2] = FPlane(CascadeFrustumVerts[0], CascadeFrustumVerts[4], CascadeFrustumVerts[5]); // Right
		SubFrustumPlanes[3] = FPlane(CascadeFrustumVerts[2], CascadeFrustumVerts[6], CascadeFrustumVerts[4]); // Top
		SubFrustumPlanes[4] = FPlane(CascadeFrustumVerts[5], CascadeFrustumVerts[7], CascadeFrustumVerts[3]); // Bottom
		SubFrustumPlanes[5] = FPlane(CascadeFrustumVerts[4], CascadeFrustumVerts[6], CascadeFrustumVerts[7]); // Far
	}
	else
	{
		SubFrustumPlanes[0] = FPlane(CascadeFrustumVerts[0], CascadeFrustumVerts[2], CascadeFrustumVerts[3]); // Near
		SubFrustumPlanes[1] = FPlane(CascadeFrustumVerts[2], CascadeFrustumVerts[6], CascadeFrustumVerts[7]); // Left
		SubFrustumPlanes[2] = FPlane(CascadeFrustumVerts[5], CascadeFrustumVerts[4], CascadeFrustumVerts[0]); // Right
		SubFrustumPlanes[3] = FPlane(CascadeFrustumVerts[4], CascadeFrustumVerts[6], CascadeFrustumVerts[2]); // Top
		SubFrustumPlanes[4] = FPlane(CascadeFrustumVerts[3], CascadeFrustumVerts[7], CascadeFrustumVerts[5]); // Bottom
		SubFrustumPlanes[5] = FPlane(CascadeFrustumVerts[7], CascadeFrustumVerts[6], CascadeFrustumVerts[4]); // Far
	}

	NearPlaneOut = SubFrustumPlanes[0];
	FarPlaneOut = SubFrustumPlanes[5];

	// Add the planes from the camera's frustum which form the back face of the frustum when in light space.
	for (int32 i = 0; i < 6; i++)
	{
		FVector Normal(SubFrustumPlanes[i]);
		float d = Normal | LightDirection;
		if (d < 0.0f)
		{
			Planes.Add(SubFrustumPlanes[i]);
		}
	}

	// Now add the planes which form the silhouette edges of the camera frustum in light space.
	for (int32 i = 0; i < 12; i++)
	{
		FVector NormalA(SubFrustumPlanes[AdjacentPlanePairs[i][0]]);
		FVector NormalB(SubFrustumPlanes[AdjacentPlanePairs[i][1]]);

		float DotA = NormalA | LightDirection;
		float DotB = NormalB | LightDirection;

		// If the signs of the dot product are different
		if (DotA * DotB < 0.0f)
		{
			// Planes are opposing, so this is an edge. 
			// Extrude the plane along the light direction, and add it to the array.

			FVector A = CascadeFrustumVerts[LineVertexIndices[i][0]];
			FVector B = CascadeFrustumVerts[LineVertexIndices[i][1]];
			// Scale the 3rd vector by the length of AB for precision
			FVector C = A + LightDirection * (A - B).Size();

			// Account for winding
			if ((DotA >= 0.0f) != bReverseCulling)
			{
				Planes.Add(FPlane(A, B, C));
			}
			else
			{
				Planes.Add(FPlane(B, A, C));
			}
		}
	}
	ConvexVolumeOut = FConvexVolume(Planes);
}

float ComputeWholeSceneDynamicShadowRadius(EComponentMobility::Type Mobility, float DynamicShadowDistanceMovableLight, float DynamicShadowDistanceStationaryLight)
{
	const bool bAllowStaticLighting = IsStaticLightingAllowed();

	if (Mobility == EComponentMobility::Movable || !bAllowStaticLighting)
	{
		return DynamicShadowDistanceMovableLight;
	}
	else
	{
		return DynamicShadowDistanceStationaryLight;
	}
}

/**
 * The scene info for a directional light.
 */
class FDirectionalLightSceneProxy : public FLightSceneProxy
{
public:

	/** Control how the cascade size influence the shadow depth bias. */
	float ShadowCascadeBiasDistribution;

	/** Whether to occlude fog and atmosphere inscattering with screenspace blurred occlusion from this light. */
	bool bEnableLightShaftOcclusion;

	bool bUseInsetShadowsForMovableObjects;

	/** 
	 * Controls how dark the occlusion masking is, a value of 1 results in no darkening term.
	 */
	float OcclusionMaskDarkness;

	/** Everything closer to the camera than this distance will occlude light shafts. */
	float OcclusionDepthRange;

	/** 
	 * Can be used to make light shafts come from somewhere other than the light's actual direction. 
	 * Will only be used when non-zero.
	 */
	FVector LightShaftOverrideDirection;

	/**
	 * Whether or not the light uses per pixel transmittance, driving the fact that should apply the simple transmittance computed on CPU during lighting pass. 
	 * If per pixel transmittance is enabled, it should not be done to avoid double transmittance contribution (transmittance will be applied on GPU).
	 */
	bool bPerPixelTransmittanceEnabled;

	/**
	 * The luminance of the sun disk in space (function of the sun illuminance and solid angle)
	 */
	FLinearColor SunDiscOuterSpaceLuminance;

	/**
	 * The atmosphere transmittance from ground to current light direction.
	 */
	FLinearColor TransmittanceTowardSun;

	/** 
	 * Radius of the whole scene dynamic shadow centered on the viewer, which replaces the precomputed shadows based on distance from the camera.  
	 * A Radius of 0 disables the dynamic shadow.
	 */
	float WholeSceneDynamicShadowRadius;

	/** 
	 * Number of cascades to split the view frustum into for the whole scene dynamic shadow.  
	 * More cascades result in better shadow resolution and allow WholeSceneDynamicShadowRadius to be further, but add rendering cost.
	 */
	uint32 DynamicShadowCascades;

	/** 
	 * Exponent that is applied to the cascade transition distances as a fraction of WholeSceneDynamicShadowRadius.
	 * An exponent of 1 means that cascade transitions will happen at a distance proportional to their resolution.
	 * A value greater than 1 brings transitions closer to the camera.
	 */
	float CascadeDistributionExponent;

	/** see UDirectionalLightComponent::CascadeTransitionFraction */
	float CascadeTransitionFraction;

	/** see UDirectionalLightComponent::ShadowDistanceFadeoutFraction */
	float ShadowDistanceFadeoutFraction;

	/** If greater than WholeSceneDynamicShadowRadius, a cascade will be created to support ray traced distance field shadows covering up to this distance. */
	float DistanceFieldShadowDistance;

	/** Forward lighting priority for the single slot available for a single directional light.*/
	int32 ForwardShadingPriority;

	/** Light source angle in degrees. */
	float LightSourceAngle;

	/** Light source angle in degrees. */
	float LightSourceSoftAngle;

	/** Shadow source angle factor. */
	float ShadowSourceAngleFactor;

	/** Determines how far shadows can be cast, in world units.  Larger values increase the shadowing cost. */
	float TraceDistance;

	uint32 bCastShadowsOnClouds : 1;
	uint32 bCastShadowsOnAtmosphere : 1;
	uint32 bCastCloudShadows : 1;
	float CloudShadowExtent;
	float CloudShadowStrength;
	float CloudShadowOnAtmosphereStrength;
	float CloudShadowOnSurfaceStrength;
	float CloudShadowDepthBias;
	float CloudShadowMapResolutionScale;
	float CloudShadowRaySampleCountScale;
	FLinearColor CloudScatteredLuminanceScale;
	uint32 bPerPixelAtmosphereTransmittance : 1;

	/** Initialization constructor. */
	FDirectionalLightSceneProxy(const UDirectionalLightComponent* Component) :
		FLightSceneProxy(Component),
		ShadowCascadeBiasDistribution(Component->ShadowCascadeBiasDistribution),
		bEnableLightShaftOcclusion(Component->bEnableLightShaftOcclusion),
		bUseInsetShadowsForMovableObjects(Component->bUseInsetShadowsForMovableObjects),
		OcclusionMaskDarkness(Component->OcclusionMaskDarkness),
		OcclusionDepthRange(Component->OcclusionDepthRange),
		LightShaftOverrideDirection(Component->LightShaftOverrideDirection),
		bPerPixelTransmittanceEnabled(Component->bPerPixelAtmosphereTransmittance),
		SunDiscOuterSpaceLuminance(FLinearColor::White),
		TransmittanceTowardSun(FLinearColor::White),
		DynamicShadowCascades(Component->DynamicShadowCascades > 0 ? Component->DynamicShadowCascades : 0),
		CascadeDistributionExponent(Component->CascadeDistributionExponent),
		CascadeTransitionFraction(Component->CascadeTransitionFraction),
		ShadowDistanceFadeoutFraction(Component->ShadowDistanceFadeoutFraction),
		DistanceFieldShadowDistance(Component->bUseRayTracedDistanceFieldShadows ? Component->DistanceFieldShadowDistance : 0),
		ForwardShadingPriority(Component->ForwardShadingPriority),
		LightSourceAngle(Component->LightSourceAngle),
		LightSourceSoftAngle(Component->LightSourceSoftAngle),
		ShadowSourceAngleFactor(Component->ShadowSourceAngleFactor),
		TraceDistance(FMath::Clamp(Component->TraceDistance, 1000.0f, 1000000.0f)),
		bCastShadowsOnClouds(Component->bCastShadowsOnClouds),
		bCastShadowsOnAtmosphere(Component->bCastShadowsOnAtmosphere),
		bCastCloudShadows(Component->bCastCloudShadows),
		CloudShadowExtent(Component->CloudShadowExtent),
		CloudShadowStrength(Component->CloudShadowStrength),
		CloudShadowOnAtmosphereStrength(Component->CloudShadowOnAtmosphereStrength),
		CloudShadowOnSurfaceStrength(Component->CloudShadowOnSurfaceStrength),
		CloudShadowDepthBias(Component->CloudShadowDepthBias),
		CloudShadowMapResolutionScale(Component->CloudShadowMapResolutionScale),
		CloudShadowRaySampleCountScale(Component->CloudShadowRaySampleCountScale),
		CloudScatteredLuminanceScale(Component->CloudScatteredLuminanceScale),
		bPerPixelAtmosphereTransmittance(Component->bPerPixelAtmosphereTransmittance)
	{
		LightShaftOverrideDirection.Normalize();

		WholeSceneDynamicShadowRadius = ComputeWholeSceneDynamicShadowRadius(Component->Mobility, Component->DynamicShadowDistanceMovableLight, Component->DynamicShadowDistanceStationaryLight);

		const float FarCascadeSize = Component->FarShadowDistance - WholeSceneDynamicShadowRadius;

		// 100 is just some number to avoid cascades of 0 size, the user still can still create many tiny cascades (very inefficient)
		if(Component->FarShadowCascadeCount && FarCascadeSize > 100.0f)
		{
			FarShadowDistance = Component->FarShadowDistance;
			FarShadowCascadeCount = Component->FarShadowCascadeCount;
		}

		{
			const FSceneInterface* Scene = Component->GetScene();
			if (Scene && IsMobilePlatform(Scene->GetShaderPlatform()))
			{
				// mobile uses whole-scene CSM for dynamic objects whenever modulated shadows are disabled
				bUseWholeSceneCSMForMovableObjects = Component->Mobility == EComponentMobility::Stationary && !Component->bCastModulatedShadows;
			}
			else
			{
				// ensure bUseWholeSceneCSMForMovableObjects is only be used with the forward renderer.
				const bool bUsingDeferredRenderer = Scene == nullptr ? true : Scene->GetShadingPath() == EShadingPath::Deferred;
				bUseWholeSceneCSMForMovableObjects = Component->Mobility == EComponentMobility::Stationary && !Component->bUseInsetShadowsForMovableObjects && !bUsingDeferredRenderer;
			}
			// Modulated shadow is only supported on mobile forward
			bCastModulatedShadows = Component->bCastModulatedShadows && (Scene == nullptr || !IsMobileDeferredShadingEnabled(Scene->GetShaderPlatform()));
		}
		
		ModulatedShadowColor = FLinearColor(Component->ModulatedShadowColor);
		ShadowAmount = Component->ShadowAmount;
	}

	void SetAtmosphereSunDiskColorScale(FLinearColor NewValue)
	{
		AtmosphereSunDiskColorScale = NewValue;
	}

	/** Accesses parameters needed for rendering the light. */
	virtual void GetLightShaderParameters(FLightRenderParameters& LightParameters, uint32 Flags=0) const override
	{
		LightParameters.WorldPosition = FVector::ZeroVector;
		LightParameters.InvRadius = 0.0f;
		LightParameters.FalloffExponent = 0.0f;

		LightParameters.Color = GetSunIlluminanceAccountingForSkyAtmospherePerPixelTransmittance();

		LightParameters.Direction = (FVector3f)-GetDirection();
		LightParameters.Tangent = (FVector3f)-GetDirection();

		LightParameters.SpotAngles = FVector2f::ZeroVector;
		LightParameters.SpecularScale = FMath::Clamp(SpecularScale, 0.f, 1.f);
		LightParameters.DiffuseScale = FMath::Clamp(DiffuseScale, 0.f, 1.f);
		LightParameters.SourceRadius = FMath::Sin( 0.5f * FMath::DegreesToRadians( LightSourceAngle ) );
		LightParameters.SoftSourceRadius = FMath::Sin( 0.5f * FMath::DegreesToRadians( LightSourceSoftAngle ) );
		LightParameters.SourceLength = 0.0f;
		LightParameters.RectLightBarnCosAngle = 0.0f;
		LightParameters.RectLightBarnLength = -2.0f;
		LightParameters.RectLightAtlasUVOffset = FVector2f::ZeroVector;
		LightParameters.RectLightAtlasUVScale = FVector2f::ZeroVector;
		LightParameters.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();
		LightParameters.IESAtlasIndex = INDEX_NONE;
		LightParameters.LightFunctionAtlasLightIndex = GetLightFunctionAtlasLightIndex();
		LightParameters.bAffectsTranslucentLighting = AffectsTranslucentLighting() ? 1 : 0;
		LightParameters.InverseExposureBlend = 0.0f;
	}

	virtual float GetLightSourceAngle() const override
	{
		return LightSourceAngle;
	}

	virtual float GetShadowSourceAngleFactor() const override
	{
		return ShadowSourceAngleFactor;
	}

	virtual float GetTraceDistance() const override
	{
		return TraceDistance;
	}

	virtual bool GetLightShaftOcclusionParameters(float& OutOcclusionMaskDarkness, float& OutOcclusionDepthRange) const override
	{
		OutOcclusionMaskDarkness = OcclusionMaskDarkness;
		OutOcclusionDepthRange = OcclusionDepthRange;
		return bEnableLightShaftOcclusion;
	}

	virtual FVector GetLightPositionForLightShafts(FVector ViewOrigin) const override
	{
		const FVector EffectiveDirection = LightShaftOverrideDirection.SizeSquared() > 0 ? LightShaftOverrideDirection : GetDirection();
		return ViewOrigin - EffectiveDirection * WORLD_MAX;
	}

	// FLightSceneInfo interface.

	virtual bool ShouldCreatePerObjectShadowsForDynamicObjects() const override
	{
		return FLightSceneProxy::ShouldCreatePerObjectShadowsForDynamicObjects()
			// Only create per-object shadows for dynamic objects if the CSM range is under some threshold
			// When the CSM range is very small, CSM is just being used to provide high resolution / animating shadows near the player,
			// But dynamic objects outside the CSM range would not have a shadow (or ones inside the range that cast a shadow out of the CSM area of influence).
			&& WholeSceneDynamicShadowRadius < GMaxCSMRadiusToAllowPerObjectShadows
			&& bUseInsetShadowsForMovableObjects && !bUseWholeSceneCSMForMovableObjects;
	}

	/** Whether this light should create CSM for dynamic objects only (mobile renderer) */
	virtual bool UseCSMForDynamicObjects() const override
	{
		return	FLightSceneProxy::ShouldCreatePerObjectShadowsForDynamicObjects()
				&& bUseWholeSceneCSMForMovableObjects
				&& WholeSceneDynamicShadowRadius > 0;
	}

	/** Returns the number of view dependent shadows this light will create, not counting distance field shadow cascades. */
	virtual uint32 GetNumViewDependentWholeSceneShadows(const FSceneView& View, bool bPrecomputedLightingIsValid) const override
	{
		uint32 ClampedFarShadowCascadeCount = FMath::Min((uint32)CVarMaxNumFarShadowCascades.GetValueOnAnyThread(), FarShadowCascadeCount);

		uint32 TotalCascades = GetNumShadowMappedCascades(View.MaxShadowCascades, bPrecomputedLightingIsValid) + ClampedFarShadowCascadeCount;

		return TotalCascades;
	}

	/**
	 * Sets up a projected shadow initializer that's dependent on the current view for shadows from the entire scene.
	 * @param InCascadeIndex ShadowSplitIndex or INDEX_NONE for the the DistanceFieldCascade
	 * @return True if the whole-scene projected shadow should be used.
	 */
	virtual bool GetViewDependentWholeSceneProjectedShadowInitializer(const FSceneView& View, int32 InCascadeIndex, bool bPrecomputedLightingIsValid, FWholeSceneProjectedShadowInitializer& OutInitializer) const override
	{
		const bool bRayTracedCascade = (InCascadeIndex == INDEX_NONE);

		FSphere Bounds = FDirectionalLightSceneProxy::GetShadowSplitBounds(View, InCascadeIndex, bPrecomputedLightingIsValid, &OutInitializer.CascadeSettings);
		// Round up to whole unit, to increase stability.
		Bounds.W = FMath::CeilToFloat(Bounds.W);

		uint32 NumNearCascades = GetNumShadowMappedCascades(View.MaxShadowCascades, bPrecomputedLightingIsValid);

		// Last cascade is the ray traced shadow, if enabled
		OutInitializer.CascadeSettings.ShadowSplitIndex = bRayTracedCascade ? NumNearCascades : InCascadeIndex;

		const float ShadowExtent = Bounds.W / FMath::Sqrt(3.0f);
		const FBoxSphereBounds SubjectBounds(Bounds.Center, FVector(ShadowExtent, ShadowExtent, ShadowExtent), Bounds.W);
		OutInitializer.PreShadowTranslation = -Bounds.Center;
		OutInitializer.WorldToLight = FInverseRotationMatrix(GetDirection().GetSafeNormal().Rotation());
		OutInitializer.Scales = FVector2D(1.0f / Bounds.W,1.0f / Bounds.W);
		OutInitializer.SubjectBounds = FBoxSphereBounds(FVector::ZeroVector,SubjectBounds.BoxExtent,SubjectBounds.SphereRadius);
		OutInitializer.WAxis = FVector4(0,0,0,1);
		// Use the minimum of half the world, things further away do not cast shadows,
		// However, if the cascade bounds are larger, then extend the casting distance far enough to encompass the cascade.
		OutInitializer.MinLightW = FMath::Min<float>(-UE_OLD_HALF_WORLD_MAX, -SubjectBounds.SphereRadius);
		// Range must extend to end of cascade bounds
		const float MaxLightW = SubjectBounds.SphereRadius;
		OutInitializer.MaxDistanceToCastInLightW = MaxLightW - OutInitializer.MinLightW;
		OutInitializer.bRayTracedDistanceField = bRayTracedCascade;
		OutInitializer.CascadeSettings.bFarShadowCascade = !bRayTracedCascade && OutInitializer.CascadeSettings.ShadowSplitIndex >= (int32)NumNearCascades;
		return true;
	}

	virtual FVector2D GetDirectionalLightDistanceFadeParameters(ERHIFeatureLevel::Type InFeatureLevel, bool bPrecomputedLightingIsValid, int32 MaxNearCascades) const override
	{
		// Virtual shadow maps do not use cascade distance fading
		if (UseVirtualShadowMaps())
		{
			return FVector2D(0, 0);
		}

		float FarDistance = GetCSMMaxDistance(bPrecomputedLightingIsValid, MaxNearCascades);
		{
			if (ShouldCreateRayTracedCascade(InFeatureLevel, bPrecomputedLightingIsValid, MaxNearCascades))
			{
				FarDistance = GetDistanceFieldShadowDistance();
			}
			FarDistance = FMath::Max(FarDistance, CVarFarShadowDistanceOverride.GetValueOnAnyThread() > 0.0f ? CVarFarShadowDistanceOverride.GetValueOnAnyThread() : FarShadowDistance);
		}
	    
		// The far distance for the dynamic to static fade is the range of the directional light.
		// The near distance is placed at a depth of 90% of the light's range.
		const float NearDistance = FarDistance - FarDistance * ( ShadowDistanceFadeoutFraction * CVarCSMShadowDistanceFadeoutMultiplier.GetValueOnAnyThread() );
		return FVector2D(NearDistance, 1.0f / FMath::Max<float>(FarDistance - NearDistance, UE_KINDA_SMALL_NUMBER));
	}

	virtual int32 GetDirectionalLightForwardShadingPriority() const override
	{
		return ForwardShadingPriority;
	}

	virtual bool GetPerObjectProjectedShadowInitializer(const FBoxSphereBounds& SubjectBounds,FPerObjectProjectedShadowInitializer& OutInitializer) const override
	{
		OutInitializer.PreShadowTranslation = -SubjectBounds.Origin;
		OutInitializer.WorldToLight = FInverseRotationMatrix(FVector(WorldToLight.M[0][0],WorldToLight.M[1][0],WorldToLight.M[2][0]).GetSafeNormal().Rotation());
		OutInitializer.Scales = FVector2D(1.0f / SubjectBounds.SphereRadius,1.0f / SubjectBounds.SphereRadius);
		OutInitializer.SubjectBounds = FBoxSphereBounds(FVector::ZeroVector,SubjectBounds.BoxExtent,SubjectBounds.SphereRadius);
		OutInitializer.WAxis = FVector4(0,0,0,1);
		OutInitializer.MinLightW = -UE_OLD_HALF_WORLD_MAX;
		// Reduce casting distance on a directional light
		// This is necessary to improve floating point precision in several places, especially when deriving frustum verts from InvReceiverMatrix
		// This takes the object size into account to ensure that large objects get an extended distance
		OutInitializer.MaxDistanceToCastInLightW = FMath::Clamp(SubjectBounds.SphereRadius * CVarPerObjectCastDistanceRadiusScale.GetValueOnRenderThread(), CVarPerObjectCastDistanceMin.GetValueOnRenderThread(), (float)WORLD_MAX);

		return true;
	}

	virtual bool ShouldCreateRayTracedCascade(ERHIFeatureLevel::Type InFeatureLevel, bool bPrecomputedLightingIsValid, int32 MaxNearCascades) const override
	{
		static auto CVarDistanceFieldShadowing = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DistanceFieldShadowing"));
		static IConsoleVariable* CVarHFShadowing = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HeightFieldShadowing"));

		if (CVarDistanceFieldShadowing != nullptr && CVarDistanceFieldShadowing->GetInt() == 0 && (!CVarHFShadowing || !CVarHFShadowing->GetInt()))
		{
			return false;
		}

		const uint32 NumCascades = GetNumShadowMappedCascades(MaxNearCascades, bPrecomputedLightingIsValid);
		const float RayTracedShadowDistance = GetDistanceFieldShadowDistance();
		const bool bCreateWithCSM = NumCascades > 0 && RayTracedShadowDistance > GetCSMMaxDistance(bPrecomputedLightingIsValid, MaxNearCascades);
		const bool bCreateWithoutCSM = NumCascades == 0 && RayTracedShadowDistance > 0;
		return DoesPlatformSupportDistanceFieldShadowing(GShaderPlatformForFeatureLevel[InFeatureLevel]) && (bCreateWithCSM || bCreateWithoutCSM);
	}

	virtual void SetAtmosphereRelatedProperties(FLinearColor TransmittanceTowardSunIn, FLinearColor SunDiscOuterSpaceLuminanceIn) override
	{
		TransmittanceTowardSun = TransmittanceTowardSunIn;
		SunDiscOuterSpaceLuminance = SunDiscOuterSpaceLuminanceIn;
	}

	virtual FLinearColor GetOuterSpaceLuminance() const override
	{ 
		return SunDiscOuterSpaceLuminance;
	}

	virtual FLinearColor GetOuterSpaceIlluminance() const override
	{
		return GetColor();
	}

	virtual FLinearColor GetAtmosphereTransmittanceTowardSun() const override
	{
		return TransmittanceTowardSun;
	}

	virtual FLinearColor GetSunIlluminanceOnGroundPostTransmittance() const override
	{
		return GetOuterSpaceIlluminance() * GetAtmosphereTransmittanceTowardSun();
	}

	virtual bool GetPerPixelTransmittanceEnabled() const override
	{
		return bPerPixelTransmittanceEnabled;
	}

	virtual FLinearColor GetSunIlluminanceAccountingForSkyAtmospherePerPixelTransmittance() const override
	{
		if (IsUsedAsAtmosphereSunLight())
		{
			// We only apply transmittance in some cases. For instance if transmittance is evaluated per pixel, we do not apply it to the light illuminance.
			if (bPerPixelTransmittanceEnabled)
			{
				return FLinearColor(FVector(GetOuterSpaceIlluminance()));
			}
			else
			{
				return FLinearColor(FVector(GetOuterSpaceIlluminance() * GetAtmosphereTransmittanceTowardSun()));
			}
		}
		return GetColor();
	}

	virtual float GetSunLightHalfApexAngleRadian() const override
	{
		return 0.5f * LightSourceAngle * UE_PI / 180.0f; // LightSourceAngle is apex angle (angular diameter) in degree
	}


	virtual bool GetCastShadowsOnClouds()  const override 
	{ 
		return bCastShadowsOnClouds; 
	}
	virtual bool GetCastShadowsOnAtmosphere()  const override
	{
		return bCastShadowsOnAtmosphere;
	}
	virtual bool GetCastCloudShadows()  const override
	{
		return bCastCloudShadows;
	}
	virtual float GetCloudShadowExtent()  const override
	{
		return CloudShadowExtent;
	}
	virtual float GetCloudShadowMapResolutionScale()  const override
	{
		return CloudShadowMapResolutionScale;
	}
	virtual float GetCloudShadowRaySampleCountScale()  const override
	{
		return CloudShadowRaySampleCountScale;
	}
	virtual float GetCloudShadowStrength()  const override
	{
		return CloudShadowStrength;
	}
	virtual float GetCloudShadowOnAtmosphereStrength()  const override
	{
		return CloudShadowOnAtmosphereStrength;
	}
	virtual float GetCloudShadowDepthBias()  const override
	{
		return CloudShadowDepthBias;
	}
	virtual float GetCloudShadowOnSurfaceStrength()  const override
	{
		return CloudShadowOnSurfaceStrength;
	}
	virtual FLinearColor GetCloudScatteredLuminanceScale()  const override
	{
		return CloudScatteredLuminanceScale;
	}
	virtual bool GetUsePerPixelAtmosphereTransmittance()  const override
	{
		return bPerPixelAtmosphereTransmittance;
	}

private:

	float GetEffectiveWholeSceneDynamicShadowRadius(bool bPrecomputedLightingIsValid) const
	{
		return bPrecomputedLightingIsValid ? WholeSceneDynamicShadowRadius : CVarUnbuiltWholeSceneDynamicShadowRadius.GetValueOnAnyThread();
	}

	uint32 GetNumShadowMappedCascades(uint32 MaxShadowCascades, bool bPrecomputedLightingIsValid) const
	{
		int32 EffectiveNumDynamicShadowCascades = DynamicShadowCascades;
		if (!bPrecomputedLightingIsValid)
		{
			EffectiveNumDynamicShadowCascades = FMath::Max(0, CVarUnbuiltNumWholeSceneDynamicShadowCascades.GetValueOnAnyThread());

			static const auto CVarUnbuiltPreviewShadowsInGame = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.UnbuiltPreviewInGame"));
			bool bUnbuiltPreviewShadowsInGame = CVarUnbuiltPreviewShadowsInGame->GetInt() != 0;

			if (!bUnbuiltPreviewShadowsInGame && !GetSceneInterface()->IsEditorScene())
			{
				EffectiveNumDynamicShadowCascades = 0;
			}
		}

		const int32 NumCascades = GetCSMMaxDistance(bPrecomputedLightingIsValid, MaxShadowCascades) > 0.0f ? EffectiveNumDynamicShadowCascades : 0;
		return FMath::Min<int32>(NumCascades, MaxShadowCascades);
	}

	float GetCSMMaxDistance(bool bPrecomputedLightingIsValid, int32 MaxShadowCascades) const
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.Shadow.DistanceScale"));
		
		if (MaxShadowCascades <= 0)
		{
			return 0.0f;
		}

		float Scale = FMath::Clamp(CVar->GetValueOnRenderThread(), 0.0f, 2.0f);
		float Distance = GetEffectiveWholeSceneDynamicShadowRadius(bPrecomputedLightingIsValid) * Scale;
		return Distance;
	}

	virtual float GetMaxShadowReceiverDistance(const FSceneView& View, bool bPrecomputedLightingIsValid) const override
	{
		const float CSMMax = GetCSMMaxDistance(bPrecomputedLightingIsValid, View.MaxShadowCascades);
		const uint32 ClampedFarCount = FMath::Min((uint32)CVarMaxNumFarShadowCascades.GetValueOnAnyThread(), FarShadowCascadeCount);
		if (ClampedFarCount > 0)
		{
			const float FarOverride = CVarFarShadowDistanceOverride.GetValueOnAnyThread();
			const float FarEnd = FarOverride > 0.0f ? FarOverride : FarShadowDistance;
			return FMath::Max(CSMMax, FarEnd);
		}
		return CSMMax;
	}

	float GetDistanceFieldShadowDistance() const
	{
		if (!DoesProjectSupportDistanceFields())
		{
			// Meshes must have distance fields generated
			return 0;
		}

		return DistanceFieldShadowDistance * FMath::Clamp(CVarRTDFDistanceScale.GetValueOnRenderThread(), 0.0001f, 10000.0f);
	}

	float GetShadowTransitionScale() const
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.Shadow.CSM.TransitionScale"));
		float Scale = FMath::Clamp(CVar->GetValueOnRenderThread(), 0.0f, 2.0f);
		return Scale;
	}

	void UpdateLightShaftOverrideDirection_RenderThread(FVector NewLightShaftOverrideDirection)
	{
		LightShaftOverrideDirection = NewLightShaftOverrideDirection;
	}

	// Useful helper function to compute shadow map cascade distribution
	// @param Exponent >=1, 1:linear, 2:each cascade gets 2x in size, ...
	// @param CascadeIndex 0..CascadeCount
	// @param CascadeCount >0
	static float ComputeAccumulatedScale(float Exponent, int32 CascadeIndex, int32 CascadeCount)
	{
		if(CascadeIndex <= 0)
		{
			return 0.0f;
		}

		float CurrentScale = 1;
		float TotalScale = 0.0f;
		float Ret = 0.0f;

		// lame implementation for simplicity, CascadeIndex is small anyway
		for(int i = 0; i < CascadeCount; ++i)
		{
			if(i < CascadeIndex)
			{
				Ret += CurrentScale;
			}
			TotalScale += CurrentScale;
			CurrentScale *= Exponent;
		}

		return Ret / TotalScale;
	}
	
	float GetEffectiveCascadeDistributionExponent(bool bPrecomputedLightingIsValid) const
	{
		return bPrecomputedLightingIsValid ? CascadeDistributionExponent : 4;
	}

	// @param SplitIndex 0: near, 1:end of 1st cascade, ...
	inline float GetSplitDistance(const FSceneView& View, uint32 SplitIndex, bool bPrecomputedLightingIsValid, bool bDistanceFieldShadows) const
	{
		// near cascade means non far and non distance field cascade
		uint32 NumNearCascades = GetNumShadowMappedCascades(View.MaxShadowCascades, bPrecomputedLightingIsValid);

		float CascadeDistanceWithoutFar = GetCSMMaxDistance(bPrecomputedLightingIsValid, View.MaxShadowCascades);
		float ShadowNear = View.NearClippingDistance;
		const float EffectiveCascadeDistributionExponent = GetEffectiveCascadeDistributionExponent(bPrecomputedLightingIsValid);

		// non near cascades are split differently for distance field shadow
		if(SplitIndex > NumNearCascades)
		{
			if(bDistanceFieldShadows)
			{
				// there is only one distance field shadow cascade
				check(SplitIndex == NumNearCascades + 1);
				return GetDistanceFieldShadowDistance();
			}
			else
			{
				// the far cascades start at the after the near cascades
				uint32 ClampedFarShadowCascadeCount = FMath::Min((uint32)CVarMaxNumFarShadowCascades.GetValueOnAnyThread(), FarShadowCascadeCount);

				return CascadeDistanceWithoutFar + ComputeAccumulatedScale(EffectiveCascadeDistributionExponent, SplitIndex - NumNearCascades, ClampedFarShadowCascadeCount) * ((CVarFarShadowDistanceOverride.GetValueOnAnyThread() > 0.0f ? CVarFarShadowDistanceOverride.GetValueOnAnyThread() : FarShadowDistance) - CascadeDistanceWithoutFar);
			}
		}
		else
		{
			return ShadowNear + ComputeAccumulatedScale(EffectiveCascadeDistributionExponent, SplitIndex, NumNearCascades) * (CascadeDistanceWithoutFar - ShadowNear);
		}
	}

	virtual FSphere GetShadowSplitBoundsDepthRange(const FSceneView& View, FVector ViewOrigin, float SplitNear, float SplitFar, FShadowCascadeSettings* OutCascadeSettings) const override
	{
		const FVector LightDirection = -GetDirection();

		if (View.bIsSceneCaptureCube && IsSceneCaptureCubeShareCascadesEnabled(View.GetShaderPlatform()))
		{
			if (OutCascadeSettings)
			{
				// Get the 8 corners of the cascade's camera frustum, in world space
				FVector CascadeFrustumVerts[8];
				CascadeFrustumVerts[0] = ViewOrigin + FVector( SplitFar,  SplitFar, -SplitFar);    // 0 Near Top    Right
				CascadeFrustumVerts[1] = ViewOrigin + FVector( SplitFar, -SplitFar, -SplitFar);    // 1 Near Bottom Right
				CascadeFrustumVerts[2] = ViewOrigin + FVector(-SplitFar,  SplitFar, -SplitFar);    // 2 Near Top    Left
				CascadeFrustumVerts[3] = ViewOrigin + FVector(-SplitFar, -SplitFar, -SplitFar);    // 3 Near Bottom Left
				CascadeFrustumVerts[4] = ViewOrigin + FVector( SplitFar,  SplitFar,  SplitFar);    // 4 Far  Top    Right
				CascadeFrustumVerts[5] = ViewOrigin + FVector( SplitFar, -SplitFar,  SplitFar);    // 5 Far  Bottom Right
				CascadeFrustumVerts[6] = ViewOrigin + FVector(-SplitFar,  SplitFar,  SplitFar);    // 6 Far  Top    Left
				CascadeFrustumVerts[7] = ViewOrigin + FVector(-SplitFar, -SplitFar,  SplitFar);    // 7 Far  Bottom Left

				ComputeShadowCullingVolume(View.bReverseCulling, CascadeFrustumVerts, LightDirection, OutCascadeSettings->ShadowBoundsAccurate, OutCascadeSettings->NearFrustumPlane, OutCascadeSettings->FarFrustumPlane);
			}

			// Set a sphere around the entire box bounding volume of all six cube map faces
			const float Sqrt3 = 1.73205080756887729353f;
			return FSphere(ViewOrigin, Sqrt3 * SplitFar);
		}

		const FViewMatrices& ViewMatrices = View.GetShadowViewMatrices();

		const FMatrix& ViewMatrix = ViewMatrices.GetWorldToView();
		const FMatrix& ProjectionMatrix = ViewMatrices.GetViewToClipNoAA();

		const FVector CameraDirection = ViewMatrix.GetColumn(2);

		// Support asymmetric projection
		// Get FOV and AspectRatio from the view's projection matrix.
		float AspectRatio = ProjectionMatrix.M[1][1] / ProjectionMatrix.M[0][0];
		bool bIsPerspectiveProjection = ViewMatrices.IsPerspectiveProjection();

		// Build the camera frustum for this cascade
		float HalfHorizontalFOV = bIsPerspectiveProjection ? FMath::Atan(1.0f / ProjectionMatrix.M[0][0]) : UE_PI / 4.0f;
		float HalfVerticalFOV = bIsPerspectiveProjection ? FMath::Atan(1.0f / ProjectionMatrix.M[1][1]) : FMath::Atan((FMath::Tan(UE_PI / 4.0f) / AspectRatio));
		float AsymmetricFOVScaleX = ProjectionMatrix.M[2][0];
		float AsymmetricFOVScaleY = ProjectionMatrix.M[2][1];

		// Near plane
		const float StartHorizontalTotalLength = SplitNear * FMath::Tan(HalfHorizontalFOV);
		const float StartVerticalTotalLength = SplitNear * FMath::Tan(HalfVerticalFOV);
		const FVector StartCameraLeftOffset = ViewMatrix.GetColumn(0) * -StartHorizontalTotalLength * (1 + AsymmetricFOVScaleX);
		const FVector StartCameraRightOffset = ViewMatrix.GetColumn(0) *  StartHorizontalTotalLength * (1 - AsymmetricFOVScaleX);
		const FVector StartCameraBottomOffset = ViewMatrix.GetColumn(1) * -StartVerticalTotalLength * (1 + AsymmetricFOVScaleY);
		const FVector StartCameraTopOffset = ViewMatrix.GetColumn(1) *  StartVerticalTotalLength * (1 - AsymmetricFOVScaleY);
		// Far plane
		const float EndHorizontalTotalLength = SplitFar * FMath::Tan(HalfHorizontalFOV);
		const float EndVerticalTotalLength = SplitFar * FMath::Tan(HalfVerticalFOV);
		const FVector EndCameraLeftOffset = ViewMatrix.GetColumn(0) * -EndHorizontalTotalLength * (1 + AsymmetricFOVScaleX);
		const FVector EndCameraRightOffset = ViewMatrix.GetColumn(0) *  EndHorizontalTotalLength * (1 - AsymmetricFOVScaleX);
		const FVector EndCameraBottomOffset = ViewMatrix.GetColumn(1) * -EndVerticalTotalLength * (1 + AsymmetricFOVScaleY);
		const FVector EndCameraTopOffset = ViewMatrix.GetColumn(1) *  EndVerticalTotalLength * (1 - AsymmetricFOVScaleY);

		// Get the 8 corners of the cascade's camera frustum, in world space
		FVector CascadeFrustumVerts[8];
		CascadeFrustumVerts[0] = ViewOrigin + CameraDirection * SplitNear + StartCameraRightOffset + StartCameraTopOffset;    // 0 Near Top    Right
		CascadeFrustumVerts[1] = ViewOrigin + CameraDirection * SplitNear + StartCameraRightOffset + StartCameraBottomOffset; // 1 Near Bottom Right
		CascadeFrustumVerts[2] = ViewOrigin + CameraDirection * SplitNear + StartCameraLeftOffset + StartCameraTopOffset;     // 2 Near Top    Left
		CascadeFrustumVerts[3] = ViewOrigin + CameraDirection * SplitNear + StartCameraLeftOffset + StartCameraBottomOffset;  // 3 Near Bottom Left
		CascadeFrustumVerts[4] = ViewOrigin + CameraDirection * SplitFar + EndCameraRightOffset + EndCameraTopOffset;       // 4 Far  Top    Right
		CascadeFrustumVerts[5] = ViewOrigin + CameraDirection * SplitFar + EndCameraRightOffset + EndCameraBottomOffset;    // 5 Far  Bottom Right
		CascadeFrustumVerts[6] = ViewOrigin + CameraDirection * SplitFar + EndCameraLeftOffset + EndCameraTopOffset;       // 6 Far  Top    Left
		CascadeFrustumVerts[7] = ViewOrigin + CameraDirection * SplitFar + EndCameraLeftOffset + EndCameraBottomOffset;    // 7 Far  Bottom Left

		// Fit a bounding sphere around the world space camera cascade frustum.
		// Compute the sphere ideal centre point given the FOV and near/far.
		float TanHalfFOVx = FMath::Tan(HalfHorizontalFOV);
		float TanHalfFOVy = FMath::Tan(HalfVerticalFOV);
		float FrustumLength = SplitFar - SplitNear;

		float FarX = TanHalfFOVx * SplitFar;
		float FarY = TanHalfFOVy * SplitFar;
		float DiagonalASq = FarX * FarX + FarY * FarY;

		float NearX = TanHalfFOVx * SplitNear;
		float NearY = TanHalfFOVy * SplitNear;
		float DiagonalBSq = NearX * NearX + NearY * NearY;

		// Calculate the ideal bounding sphere for the subfrustum.
		// Fx  = (Db^2 - da^2) / 2Fl + Fl / 2 
		// (where Da is the far diagonal, and Db is the near, and Fl is the frustum length)
		float OptimalOffset = (DiagonalBSq - DiagonalASq) / (2.0f * FrustumLength) + FrustumLength * 0.5f;
		float CentreZ = SplitFar - OptimalOffset;
		CentreZ = FMath::Clamp( CentreZ, SplitNear, SplitFar );
		FSphere CascadeSphere(ViewOrigin + CameraDirection * CentreZ, 0);
		for (int32 Index = 0; Index < 8; Index++)
		{
			CascadeSphere.W = FMath::Max(CascadeSphere.W, FVector::DistSquared(CascadeFrustumVerts[Index], CascadeSphere.Center));
		}

		// Don't allow the bounds to reach 0 (INF)
		CascadeSphere.W = FMath::Max(FMath::Sqrt(CascadeSphere.W), 1.0f); 

		if (OutCascadeSettings)
		{
			// this function is needed, since it's also called by the LPV code.
			ComputeShadowCullingVolume(View.bReverseCulling, CascadeFrustumVerts, LightDirection, OutCascadeSettings->ShadowBoundsAccurate, OutCascadeSettings->NearFrustumPlane, OutCascadeSettings->FarFrustumPlane);
		}

		return CascadeSphere;
	}

	// @param InShadowSplitIndex cascade index or InShadowSplitIndex == INDEX_NONE for the distance field cascade
	virtual FSphere GetShadowSplitBounds(const FSceneView& View, int32 InCascadeIndex, bool bPrecomputedLightingIsValid, FShadowCascadeSettings* OutCascadeSettings) const override
	{
		uint32 NumNearCascades = GetNumShadowMappedCascades(View.MaxShadowCascades, bPrecomputedLightingIsValid);

		const bool bHasRayTracedCascade = ShouldCreateRayTracedCascade(View.GetFeatureLevel(), bPrecomputedLightingIsValid, View.MaxShadowCascades);

		// this checks for WholeSceneDynamicShadowRadius and DynamicShadowCascades
		uint32 NumNearAndFarCascades = GetNumViewDependentWholeSceneShadows(View, bPrecomputedLightingIsValid);

		uint32 NumTotalCascades = FMath::Max(NumNearAndFarCascades, NumNearCascades + (bHasRayTracedCascade ? 1 : 0));

		const bool bIsRayTracedCascade = InCascadeIndex == INDEX_NONE;
		const uint32 ShadowSplitIndex = bIsRayTracedCascade ? NumNearCascades : InCascadeIndex;		// todo -1?

		// Determine start and end distances to the current cascade's split planes
		// Presence of the ray traced cascade does not change depth ranges for the shadow-mapped cascades
		float SplitNear = GetSplitDistance(View, ShadowSplitIndex, bPrecomputedLightingIsValid, bIsRayTracedCascade);
		float SplitFar = GetSplitDistance(View, ShadowSplitIndex + 1, bPrecomputedLightingIsValid, bIsRayTracedCascade);

		float FadePlane = SplitFar;

		float LocalCascadeTransitionFraction = CascadeTransitionFraction * GetShadowTransitionScale();

		float FadeExtension = (SplitFar - SplitNear) * LocalCascadeTransitionFraction;

		// Add the fade region to the end of the subfrustum, if this is not the last cascade.
		if ((int32)ShadowSplitIndex < (int32)NumTotalCascades - 1)
		{
			SplitFar += FadeExtension;
		}
		// Only do this if there is no static shadowing taking over after the fade (i.e., the intention is to fade to nothing, when only dynamic shadowing is used)
		else if (!(bPrecomputedLightingIsValid && bStaticShadowing))
		{
			if (bIsRayTracedCascade)
			{
				FadeExtension *= FMath::Clamp(CVarRtdfFarTransitionScale.GetValueOnAnyThread(), 0.0f, 1.0f);
			}
			// For the last cascade, we want to fade out to avoid a hard line, since there is no further cascade to overlap with, 
			// extending the far makes little sense as extending the shadow range would be counter intuitive and affect performance. 
			// Thus, move the fade plane closer:
			FadePlane -= FadeExtension;
		}

		if(OutCascadeSettings)
		{
			OutCascadeSettings->SplitFarFadeRegion = FadeExtension;

			OutCascadeSettings->SplitNearFadeRegion = 0.0f;

			if(ShadowSplitIndex >= 1)
			{
				// only used to fade the translucency lighting volume

				float BeforeSplitNear = GetSplitDistance(View, ShadowSplitIndex - 1, bPrecomputedLightingIsValid, bIsRayTracedCascade);
				float BeforeSplitFar = GetSplitDistance(View, ShadowSplitIndex, bPrecomputedLightingIsValid, bIsRayTracedCascade);

				OutCascadeSettings->SplitNearFadeRegion = (BeforeSplitFar - BeforeSplitNear) * LocalCascadeTransitionFraction;
			}

			// Pass out the split settings
			OutCascadeSettings->SplitFar = SplitFar;
			OutCascadeSettings->SplitNear = SplitNear;
			OutCascadeSettings->FadePlaneOffset = FadePlane;
			OutCascadeSettings->FadePlaneLength = SplitFar - FadePlane;
			OutCascadeSettings->CascadeBiasDistribution = ShadowCascadeBiasDistribution;
			OutCascadeSettings->ShadowSplitIndex = (int32)ShadowSplitIndex;
		}


		// TODO: Move this into a flag in some shadow-system cofiguration struct and set based on this cvar outside setup.
		static const auto CVarSkyAtmosphereSampleLightShadowmap = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SkyAtmosphere.SampleLightShadowmap"));

		// If enabled & this is the last cascade, then override the near plane used to construct the bounds using the
		// near plane of the first cacade (AKA the near plane of the view), to ensure the last cascade is inclusive.
		// We don't set any of the near/far planes etc as that would conflict with the fading.
		bool bUseNearOverride = (bCastShadowsOnClouds || bCastShadowsOnAtmosphere) 
			&& (ShadowSplitIndex + 1U) == NumNearAndFarCascades 
			&& CVarSkyAtmosphereSampleLightShadowmap && CVarSkyAtmosphereSampleLightShadowmap->GetValueOnAnyThread() != 0;
		
		const float BoundsCalcNear = bUseNearOverride ? GetSplitDistance(View, 0, bPrecomputedLightingIsValid, bIsRayTracedCascade) : SplitNear;
		const FSphere CascadeSphere = FDirectionalLightSceneProxy::GetShadowSplitBoundsDepthRange(View, View.ViewMatrices.GetViewOrigin(), BoundsCalcNear, SplitFar, OutCascadeSettings);

		return CascadeSphere;
	}
};

UDirectionalLightComponent::UDirectionalLightComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UTexture2D> StaticTexture(TEXT("/Engine/EditorResources/LightIcons/S_LightDirectional"));
		static ConstructorHelpers::FObjectFinder<UTexture2D> DynamicTexture(TEXT("/Engine/EditorResources/LightIcons/S_LightDirectionalMove"));

		StaticEditorTexture = StaticTexture.Object;
		StaticEditorTextureScale = 0.5f;
		DynamicEditorTexture = DynamicTexture.Object;
		DynamicEditorTextureScale = 0.5f;
	}
#endif

	Intensity = 10;

	bEnableLightShaftOcclusion = false;
	OcclusionDepthRange = 100000.f;
	OcclusionMaskDarkness = 0.05f;

	ShadowCascadeBiasDistribution = 1;
	WholeSceneDynamicShadowRadius_DEPRECATED = 20000.0f;
	DynamicShadowDistanceStationaryLight = 0.f;

	ForwardShadingPriority = 0;
	// For 5.0 double dynamic shadow distance by default & enable DF shadows.
	// Note, we reset this to old defaults in Serialize for lights created with older versions of the engine.
	DynamicShadowDistanceMovableLight = 40000.0f;
	DistanceFieldShadowDistance = 51200.0f;
	DynamicShadowCascades = 4;
	bUseRayTracedDistanceFieldShadows = true;

	TraceDistance = 10000.0f;
	FarShadowDistance = 300000.0f;
	LightSourceAngle = 0.5357f;		// Angle of earth's sun
	LightSourceSoftAngle = 0.0f;
	ShadowSourceAngleFactor = 1.0f;


	CascadeDistributionExponent = 3.0f;
	CascadeTransitionFraction = 0.1f;
	ShadowDistanceFadeoutFraction = 0.1f;
	IndirectLightingIntensity = 1.0f;
	CastTranslucentShadows = true;
	bUseInsetShadowsForMovableObjects = true;
	bCastVolumetricShadow = true;

	ModulatedShadowColor = FColor(128, 128, 128);
	ShadowAmount = 1.0f;

	bUsedAsAtmosphereSunLight_DEPRECATED = false;
	bAtmosphereSunLight = true;
	AtmosphereSunLightIndex = 0;
	AtmosphereSunDiskColorScale = FLinearColor::White;

	bCastShadowsOnClouds = 0;
	bCastShadowsOnAtmosphere = 0;
	bCastCloudShadows = 0;
	CloudShadowExtent = 150.0f;
	CloudShadowMapResolutionScale = 1.0f;
	CloudShadowRaySampleCountScale = 1.0f;
	CloudShadowStrength = 1.0f;
	CloudShadowOnAtmosphereStrength = 1.0f;
	CloudShadowOnSurfaceStrength = 1.0f;
	CloudShadowDepthBias = 0.0f;
	CloudScatteredLuminanceScale = FLinearColor::White;
	bPerPixelAtmosphereTransmittance = 0;
}

#if WITH_EDITOR
/**
 * Called after property has changed via e.g. property window or set command.
 *
 * @param	PropertyThatChanged	FProperty that has been changed, NULL if unknown
 */
void UDirectionalLightComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	LightmassSettings.LightSourceAngle = FMath::Max(LightmassSettings.LightSourceAngle, 0.0f);
	LightmassSettings.IndirectLightingSaturation = FMath::Max(LightmassSettings.IndirectLightingSaturation, 0.0f);
	LightmassSettings.ShadowExponent = FMath::Clamp(LightmassSettings.ShadowExponent, .5f, 8.0f);

	DynamicShadowDistanceMovableLight = FMath::Max(DynamicShadowDistanceMovableLight, 0.0f);
	DynamicShadowDistanceStationaryLight = FMath::Max(DynamicShadowDistanceStationaryLight, 0.0f);
	ShadowCascadeBiasDistribution = FMath::Clamp(ShadowCascadeBiasDistribution, 0.f, 1.f);

	DynamicShadowCascades = FMath::Clamp(DynamicShadowCascades, 0, 10);
	FarShadowCascadeCount = FMath::Clamp(FarShadowCascadeCount, 0, 10);
	CascadeDistributionExponent = FMath::Clamp(CascadeDistributionExponent, .1f, 10.0f);
	CascadeTransitionFraction = FMath::Clamp(CascadeTransitionFraction, UE_KINDA_SMALL_NUMBER, 0.3f);
	ShadowDistanceFadeoutFraction = FMath::Clamp(ShadowDistanceFadeoutFraction, 0.0f, 1.0f);
	// max range is larger than UI
	ShadowBias = FMath::Clamp(ShadowBias, 0.0f, 10.0f);
	ShadowSlopeBias = FMath::Clamp(ShadowSlopeBias, 0.0f, 10.0f);
	ShadowAmount = FMath::Clamp(ShadowAmount, 0.0f, 1.0f);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UDirectionalLightComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const bool bAllowStaticLighting = IsStaticLightingAllowed();

		FString PropertyName = InProperty->GetName();

		const float WholeSceneDynamicShadowRadius = ComputeWholeSceneDynamicShadowRadius(Mobility, DynamicShadowDistanceMovableLight, DynamicShadowDistanceStationaryLight);

		const bool bShadowCascades = CastShadows
			&& CastDynamicShadows 
			&& WholeSceneDynamicShadowRadius > 0
			&& Mobility != EComponentMobility::Static;

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bUseInsetShadowsForMovableObjects))
		{
			return CastShadows && CastDynamicShadows && DynamicShadowDistanceStationaryLight > 0 && Mobility == EComponentMobility::Stationary;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowDistanceMovableLight))
		{
			const bool bUseMovableLightDistance = (Mobility == EComponentMobility::Movable) || (Mobility == EComponentMobility::Stationary && !bAllowStaticLighting);
			return CastShadows && CastDynamicShadows && bUseMovableLightDistance;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowDistanceStationaryLight))
		{
			const bool bUseStationaryLightDistance = (Mobility == EComponentMobility::Stationary) && bAllowStaticLighting;
			return CastShadows && CastDynamicShadows && bUseStationaryLightDistance;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowCascades)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CascadeDistributionExponent)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CascadeTransitionFraction)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ShadowDistanceFadeoutFraction)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, FarShadowCascadeCount))
		{
			return bShadowCascades;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, FarShadowDistance))
		{
			return bShadowCascades && FarShadowCascadeCount > 0;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DistanceFieldShadowDistance)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, TraceDistance))
		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));
			bool bCanEdit = CastShadows && CastDynamicShadows && bUseRayTracedDistanceFieldShadows && Mobility != EComponentMobility::Static && CVar->GetValueOnGameThread() != 0;
			return bCanEdit;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, OcclusionMaskDarkness)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, OcclusionDepthRange))
		{
			return bEnableLightShaftOcclusion;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bCastModulatedShadows))
		{
			return bUseInsetShadowsForMovableObjects;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ModulatedShadowColor))
		{
			return bUseInsetShadowsForMovableObjects && bCastModulatedShadows;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowExtent)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowMapResolutionScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowRaySampleCountScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowStrength)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowOnAtmosphereStrength)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowOnSurfaceStrength)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowDepthBias))
		{
			return bCastCloudShadows && bAtmosphereSunLight;
		}
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bCastCloudShadows)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bCastShadowsOnAtmosphere)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudScatteredLuminanceScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bPerPixelAtmosphereTransmittance)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, AtmosphereSunLightIndex)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, AtmosphereSunDiskColorScale))
		{
			return bAtmosphereSunLight;
		}
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bCastShadowsOnClouds))
		{
			return bAtmosphereSunLight && AtmosphereSunLightIndex == 0;	// AtmosphereLight1 has opaque shadow on cloud disabled.
		}


	}

	return Super::CanEditChange(InProperty);
}

#endif // WITH_EDITOR

FLightSceneProxy* UDirectionalLightComponent::CreateSceneProxy() const
{
	if (!IsPSOPrecaching())
	{
		return new FDirectionalLightSceneProxy(this);
	}
	return nullptr;
}

FVector4 UDirectionalLightComponent::GetLightPosition() const
{
	return FVector4(-GetDirection() * WORLD_MAX, 0 );
}

/**
* @return ELightComponentType for the light component class 
*/
ELightComponentType UDirectionalLightComponent::GetLightType() const
{
	return LightType_Directional;
}

float UDirectionalLightComponent::GetUniformPenumbraSize() const
{
	if (LightmassSettings.bUseAreaShadowsForStationaryLight)
	{
		// Interpret distance as shadow factor directly
		return 1.0f;
	}
	else
	{
		// Heuristic to derive uniform penumbra size from light source angle
		return FMath::Clamp(LightmassSettings.LightSourceAngle * .05f, .0001f, 1.0f);
	}
}

void UDirectionalLightComponent::SetDynamicShadowDistanceMovableLight(float NewValue)
{
	if (DynamicShadowDistanceMovableLight != NewValue)
	{
		DynamicShadowDistanceMovableLight = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetDynamicShadowDistanceStationaryLight(float NewValue)
{
	if (DynamicShadowDistanceStationaryLight != NewValue)
	{
		DynamicShadowDistanceStationaryLight = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetDynamicShadowCascades(int32 NewValue)
{
	if (DynamicShadowCascades != NewValue)
	{
		DynamicShadowCascades = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetCascadeDistributionExponent(float NewValue)
{
	if (CascadeDistributionExponent != NewValue)
	{
		CascadeDistributionExponent = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetCascadeTransitionFraction(float NewValue)
{
	if (CascadeTransitionFraction != NewValue)
	{
		CascadeTransitionFraction = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetShadowDistanceFadeoutFraction(float NewValue)
{
	if (ShadowDistanceFadeoutFraction != NewValue)
	{
		ShadowDistanceFadeoutFraction = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetShadowCascadeBiasDistribution(float NewValue)
{
	if (ShadowCascadeBiasDistribution != NewValue)
	{
		ShadowCascadeBiasDistribution = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetEnableLightShaftOcclusion(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bEnableLightShaftOcclusion != bNewValue)
	{
		bEnableLightShaftOcclusion = bNewValue;
		UpdateProxy([bEnableLightShaftOcclusion = (bool)bEnableLightShaftOcclusion](FLightSceneProxy* Proxy)
			{
				FDirectionalLightSceneProxy* DirectionalLightSceneProxy = (FDirectionalLightSceneProxy*)Proxy;
				DirectionalLightSceneProxy->bEnableLightShaftOcclusion = bEnableLightShaftOcclusion;
			});
	}
}

void UDirectionalLightComponent::SetOcclusionMaskDarkness(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& OcclusionMaskDarkness != NewValue)
	{
		OcclusionMaskDarkness = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetOcclusionDepthRange(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& OcclusionDepthRange != NewValue)
	{
		OcclusionDepthRange = NewValue;
		MarkRenderStateDirty();
	}
}
void UDirectionalLightComponent::SetLightShaftOverrideDirection(FVector NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& LightShaftOverrideDirection != NewValue)
	{
		LightShaftOverrideDirection = NewValue;
		UpdateProxy([LightShaftOverrideDirection = LightShaftOverrideDirection](FLightSceneProxy* Proxy)
			{
				FDirectionalLightSceneProxy* DirectionalLightSceneProxy = (FDirectionalLightSceneProxy*)Proxy;
				DirectionalLightSceneProxy->LightShaftOverrideDirection = LightShaftOverrideDirection;
			});
	}
}

void UDirectionalLightComponent::SetLightSourceAngle(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& LightSourceAngle != NewValue)
	{
		LightSourceAngle = NewValue;
		UpdateProxy([LightSourceAngle = LightSourceAngle](FLightSceneProxy* Proxy)
			{
				FDirectionalLightSceneProxy* DirectionalLightSceneProxy = (FDirectionalLightSceneProxy*)Proxy;
				DirectionalLightSceneProxy->LightSourceAngle = LightSourceAngle;
			});
	}
}

void UDirectionalLightComponent::SetLightSourceSoftAngle(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& LightSourceSoftAngle != NewValue)
	{
		LightSourceSoftAngle = NewValue;
		UpdateProxy([LightSourceSoftAngle = LightSourceSoftAngle](FLightSceneProxy* Proxy)
			{
				FDirectionalLightSceneProxy* DirectionalLightSceneProxy = (FDirectionalLightSceneProxy*)Proxy;
				DirectionalLightSceneProxy->LightSourceSoftAngle = LightSourceSoftAngle;
			});
	}
}

void UDirectionalLightComponent::SetShadowSourceAngleFactor(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& ShadowSourceAngleFactor != NewValue)
	{
		ShadowSourceAngleFactor = NewValue;
		UpdateProxy([ShadowSourceAngleFactor = ShadowSourceAngleFactor](FLightSceneProxy* Proxy)
			{
				FDirectionalLightSceneProxy* DirectionalLightSceneProxy = (FDirectionalLightSceneProxy*)Proxy;
				DirectionalLightSceneProxy->ShadowSourceAngleFactor = ShadowSourceAngleFactor;
			});
	}
}

void UDirectionalLightComponent::SetShadowAmount(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& ShadowAmount != NewValue)
	{
		ShadowAmount = NewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetAtmosphereSunLight(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bAtmosphereSunLight != bNewValue)
	{
		bAtmosphereSunLight = bNewValue;
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetAtmosphereSunLightIndex(int32 NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& AtmosphereSunLightIndex != NewValue)
	{
		AtmosphereSunLightIndex = FMath::Max(0, NewValue);
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::SetAtmosphereSunDiskColorScale(FLinearColor NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& AtmosphereSunDiskColorScale != NewValue)
	{
		AtmosphereSunDiskColorScale = NewValue;
		UpdateProxy([AtmosphereSunDiskColorScale = AtmosphereSunDiskColorScale](FLightSceneProxy* Proxy)
			{
				FDirectionalLightSceneProxy* DirectionalLightSceneProxy = (FDirectionalLightSceneProxy*)Proxy;
				DirectionalLightSceneProxy->SetAtmosphereSunDiskColorScale(AtmosphereSunDiskColorScale);
			});
	}
}

void UDirectionalLightComponent::SetForwardShadingPriority(int32 NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& ForwardShadingPriority != NewValue)
	{
		ForwardShadingPriority = FMath::Max(0, NewValue);
		MarkRenderStateDirty();
	}
}

void UDirectionalLightComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		// Reset to older defaults if the light was created with a previous version of the engine & not set up by a blueprint.
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::UpdatedDirectionalLightShadowDefaults 
			&& !GetArchetype()->IsInBlueprint())
		{			
			DynamicShadowDistanceMovableLight = 20000.0f;
			DistanceFieldShadowDistance = 30000.0f;
			DynamicShadowCascades = 3;
			bUseRayTracedDistanceFieldShadows = false;
		}
	}

	Super::Serialize(Ar);

	if(Ar.UEVer() < VER_UE4_REMOVE_LIGHT_MOBILITY_CLASSES)
	{
		// If outer is a DirectionalLight, we use the ADirectionalLight::LoadedFromAnotherClass path
		if( GetOuter() != NULL && !GetOuter()->IsA(ADirectionalLight::StaticClass()) )
		{
			if(Mobility == EComponentMobility::Movable)
			{
				DynamicShadowDistanceMovableLight = WholeSceneDynamicShadowRadius_DEPRECATED; 
			}
			else if(Mobility == EComponentMobility::Stationary)
			{
				DynamicShadowDistanceStationaryLight = WholeSceneDynamicShadowRadius_DEPRECATED;
			}
		}
	}

	if ((Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DirLightsAreAtmosphereLightsByDefault))
	{
		bAtmosphereSunLight = bUsedAsAtmosphereSunLight_DEPRECATED;
	}
}

void UDirectionalLightComponent::InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly)
{
	// Directional lights don't care about translation
	if (!bTranslationOnly)
	{
		Super::InvalidateLightingCacheDetailed(bInvalidateBuildEnqueuedLighting, bTranslationOnly);
	}
}

ELightUnits UDirectionalLightComponent::GetLightUnits() const { return ELightUnits::Unitless; /* Lux */ }