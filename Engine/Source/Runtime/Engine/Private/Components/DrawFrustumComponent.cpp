// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DrawFrsutumComponent.cpp: UDrawFrsutumComponent implementation.
=============================================================================*/

#include "Components/DrawFrustumComponent.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/CollisionProfile.h"
#include "MeshElementCollector.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrawFrustumComponent)


/** Represents a draw frustum to the scene manager. */
class FDrawFrustumSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	/** 
	* Initialization constructor. 
	* @param	InComponent - game component to draw in the scene
	*/
	FDrawFrustumSceneProxy(const UDrawFrustumComponent* InComponent)
	:	FPrimitiveSceneProxy(InComponent)
	,	bFrustumEnabled(InComponent->bFrustumEnabled)
	,	FrustumColor(InComponent->FrustumColor)
	,	FrustumAngle(InComponent->FrustumAngle)
	,	FrustumAspectRatio(InComponent->FrustumAspectRatio)
	,	FrustumStartDist(InComponent->FrustumStartDist)
	,	FrustumEndDist(InComponent->FrustumEndDist)
	{		
		bWillEverBeLit = false;
	}

	// FPrimitiveSceneProxy interface.

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER( STAT_DrawFrustumSceneProxy_DrawDynamicElements );

		FVector Direction(1,0,0);
		FVector LeftVector(0,1,0);
		FVector UpVector(0,0,1);

		FVector Verts[8];

		// FOVAngle controls the horizontal angle.
		const float HozHalfAngleInRadians = FMath::DegreesToRadians(FrustumAngle * 0.5f);

		float HozLength = 0.0f;
		float VertLength = 0.0f;
		
		if (FrustumAngle > 0.0f)
		{
			HozLength = FrustumStartDist * FMath::Tan(HozHalfAngleInRadians);
			VertLength = HozLength / FrustumAspectRatio;
		}
		else
		{
			const float OrthoWidth = (FrustumAngle == 0.0f) ? 1000.0f : -FrustumAngle;
			HozLength = OrthoWidth * 0.5f;
			VertLength = HozLength / FrustumAspectRatio;
		}

		// near plane verts
		Verts[0] = (Direction * FrustumStartDist) + (UpVector * VertLength) + (LeftVector * HozLength);
		Verts[1] = (Direction * FrustumStartDist) + (UpVector * VertLength) - (LeftVector * HozLength);
		Verts[2] = (Direction * FrustumStartDist) - (UpVector * VertLength) - (LeftVector * HozLength);
		Verts[3] = (Direction * FrustumStartDist) - (UpVector * VertLength) + (LeftVector * HozLength);

		if (FrustumAngle > 0.0f)
		{
			HozLength = FrustumEndDist * FMath::Tan(HozHalfAngleInRadians);
			VertLength = HozLength / FrustumAspectRatio;
		}

		// far plane verts
		Verts[4] = (Direction * FrustumEndDist) + (UpVector * VertLength) + (LeftVector * HozLength);
		Verts[5] = (Direction * FrustumEndDist) + (UpVector * VertLength) - (LeftVector * HozLength);
		Verts[6] = (Direction * FrustumEndDist) - (UpVector * VertLength) - (LeftVector * HozLength);
		Verts[7] = (Direction * FrustumEndDist) - (UpVector * VertLength) + (LeftVector * HozLength);

		for (int32 X = 0; X < 8; ++X)
		{
			Verts[X] = GetLocalToWorld().TransformPosition(Verts[X]);
		}

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
				const FSceneView* View = Views[ViewIndex];

				const uint8 DepthPriorityGroup = GetDepthPriorityGroup(View);
				PDI->DrawLine( Verts[0], Verts[1], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[1], Verts[2], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[2], Verts[3], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[3], Verts[0], FrustumColor, DepthPriorityGroup );

				PDI->DrawLine( Verts[4], Verts[5], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[5], Verts[6], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[6], Verts[7], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[7], Verts[4], FrustumColor, DepthPriorityGroup );

				PDI->DrawLine( Verts[0], Verts[4], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[1], Verts[5], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[2], Verts[6], FrustumColor, DepthPriorityGroup );
				PDI->DrawLine( Verts[3], Verts[7], FrustumColor, DepthPriorityGroup );
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.CameraFrustums && bFrustumEnabled;
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		return Result;
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

private:
	bool bFrustumEnabled;
	FColor FrustumColor;
	float FrustumAngle;
	float FrustumAspectRatio;
	float FrustumStartDist;
	float FrustumEndDist;
};

UDrawFrustumComponent::UDrawFrustumComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FrustumColor = FColor(255, 0, 255, 255);

	FrustumAngle = 90.0f;
	FrustumAspectRatio = 1.333333f;
	FrustumStartDist = 100.0f;
	FrustumEndDist = 1000.0f;
	bUseEditorCompositing = true;
	bHiddenInGame = true;
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SetGenerateOverlapEvents(false);
}

FPrimitiveSceneProxy* UDrawFrustumComponent::CreateSceneProxy()
{
	return new FDrawFrustumSceneProxy(this);
}


FBoxSphereBounds UDrawFrustumComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds( LocalToWorld.TransformPosition(FVector::ZeroVector), FVector(FrustumEndDist), FrustumEndDist );
}

