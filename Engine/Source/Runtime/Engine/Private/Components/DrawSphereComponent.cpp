// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DrawSphereComponent.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/CollisionProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrawSphereComponent)

UDrawSphereComponent::UDrawSphereComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	bHiddenInGame = true;
	bUseEditorCompositing = true;
	SetGenerateOverlapEvents(false);
}

#if WITH_EDITOR
bool UDrawSphereComponent::ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	// Draw sphere components not treated as 'selectable' in editor
	return false;
}

bool UDrawSphereComponent::ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	// Draw sphere components not treated as 'selectable' in editor
	return false;
}
#endif

