// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HeightFogComponent.cpp: Height fog implementation.
=============================================================================*/

#include "Components/ExponentialHeightFogComponent.h"
#include "GameFramework/Info.h"
#include "StateStream/ExponentialHeightFogStateStream.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "Engine/Texture2D.h"
#include "Engine/ExponentialHeightFog.h"
#include "Net/UnrealNetwork.h"
#include "Components/BillboardComponent.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "RenderUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExponentialHeightFogComponent)

#if WITH_STATE_STREAM_ACTOR
constexpr bool UseExponentialHeightFogStateStream = true;
#endif

//#define UEXPONENTIALHEIGHTFOG_VARIABLE(Type, Prefix, Name)

#define UEXPONENTIALHEIGHTFOG_VARIABLES \
	UEXPONENTIALHEIGHTFOG_VARIABLE(FExponentialHeightFogData, , SecondFogData) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(FLinearColor, , FogInscatteringLuminance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(TObjectPtr<UTextureCube>, , InscatteringColorCubemap) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FullyDirectionalInscatteringColorDistance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(FLinearColor, , DirectionalInscatteringLuminance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(bool, b, EnableVolumetricFog) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , VolumetricFogStaticLightingScatteringIntensity) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(bool, b, OverrideLightColorsWithFogInscatteringColors) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(bool, b, Holdout) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(bool, b, RenderInMainPass) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(bool, b, VisibleInReflectionCaptures) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(bool, b, VisibleInRealTimeSkyCaptures) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , SkyLightCaptureAffectsHeightFogStrength) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , SkyLightCaptureAffectsHeightFogRoughness) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(bool, b, EnableFSSS) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FSSSSceneColorScatteringAmountScale) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FSSSSceneColorScatteringAmountPower) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FSSSSpreadScale) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FSSSBlurControl) \
	UEXPONENTIALHEIGHTFOG_VARIABLES_WITH_SETFUNCTION

#define UEXPONENTIALHEIGHTFOG_VARIABLES_WITH_SETFUNCTION \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FogDensity) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FogHeightFalloff) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(FLinearColor, , SkyAtmosphereAmbientContributionColorScale) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , InscatteringColorCubemapAngle) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(FLinearColor, , InscatteringTextureTint) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , NonDirectionalInscatteringColorDistance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , DirectionalInscatteringExponent) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , DirectionalInscatteringStartDistance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FogMaxOpacity) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , StartDistance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , EndDistance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , FogCutoffDistance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , VolumetricFogScatteringDistribution) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(FColor, , VolumetricFogAlbedo) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(FLinearColor, , VolumetricFogEmissive) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , VolumetricFogExtinctionScale) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , VolumetricFogDistance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , VolumetricFogStartDistance) \
	UEXPONENTIALHEIGHTFOG_VARIABLE(float, , VolumetricFogNearFadeInDistance) \


UExponentialHeightFogComponent::UExponentialHeightFogComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FogInscatteringColor_DEPRECATED = FLinearColor(0.447f, 0.638f, 1.0f);
	FogInscatteringLuminance = FLinearColor::Black;

	SkyAtmosphereAmbientContributionColorScale = FLinearColor::White;

	DirectionalInscatteringExponent = 4.0f;
	DirectionalInscatteringStartDistance = 10000.0f;
	DirectionalInscatteringColor_DEPRECATED = FLinearColor(0.25f, 0.25f, 0.125f);
	DirectionalInscatteringLuminance = FLinearColor::Black;

	InscatteringTextureTint = FLinearColor::White;
	FullyDirectionalInscatteringColorDistance = 100000.0f;
	NonDirectionalInscatteringColorDistance = 1000.0f;

	SkyLightCaptureAffectsHeightFogRoughness = 0.15f;
	SkyLightCaptureAffectsHeightFogStrength = 0.0f;

	FogDensity = 0.02f;
	FogHeightFalloff = 0.2f;
	// No influence from the second fog as default
	SecondFogData.FogDensity = 0.0f;

	FogMaxOpacity = 1.0f;
	StartDistance = 0.0f;
	EndDistance = 0.0f;

	// disabled by default
	FogCutoffDistance = 0;

	bHoldout = false;
	bRenderInMainPass = true;
	bVisibleInReflectionCaptures = true;
	bVisibleInRealTimeSkyCaptures = true;

	VolumetricFogScatteringDistribution = .2f;
	VolumetricFogAlbedo = FColor::White;
	VolumetricFogExtinctionScale = 1.0f;
	VolumetricFogDistance = 6000.0f;
	VolumetricFogStaticLightingScatteringIntensity = 1;

	bEnableFSSS = false;
	FSSSSceneColorScatteringAmountScale = 1.0f;
	FSSSSceneColorScatteringAmountPower = 1.0f;
	FSSSSpreadScale = 0.1f;
	FSSSBlurControl = 0.5f;
}

void UExponentialHeightFogComponent::AddFogIfNeeded()
{
	// For safety, clamp the values for SecondFogData here.
	SecondFogData.ClampToValidRanges();
	
	if (ShouldComponentAddToScene() && ShouldRender() && IsRegistered() && ((FogDensity + SecondFogData.FogDensity) * 1000) > UE_DELTA && FogMaxOpacity > UE_DELTA
		&& (GetOuter() == NULL || !GetOuter()->HasAnyFlags(RF_ClassDefaultObject)))
	{
		FExponentialHeightFogDynamicState Ds;
		#define UEXPONENTIALHEIGHTFOG_VARIABLE(Type, Prefix, Name) Ds.Set##Name(Prefix##Name);
		UEXPONENTIALHEIGHTFOG_VARIABLES
		#undef UEXPONENTIALHEIGHTFOG_VARIABLE

		Ds.SetHeight(GetComponentLocation().Z);

		#if WITH_STATE_STREAM_ACTOR
		if (UseExponentialHeightFogStateStream)
		{
			//Ds.SetComponentTransform(GetComponentTransform());
			//Ds.SetOverrideAtmosphericLight(ConvertAtmosphericLightOverride(OverrideAtmosphericLight, OverrideAtmosphericLightDirection));
			Handle = GetWorld()->GetStateStream<IExponentialHeightFogStateStream>().Game_CreateInstance({}, Ds);
			return;
		}
		#endif

		GetWorld()->Scene->AddExponentialHeightFog(uint64(this), Ds);
	}
}

void UExponentialHeightFogComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);
	AddFogIfNeeded();
}

void UExponentialHeightFogComponent::SendRenderTransform_Concurrent()
{
	GetWorld()->Scene->RemoveExponentialHeightFog(uint64(this));
	AddFogIfNeeded();
	Super::SendRenderTransform_Concurrent();
}

void UExponentialHeightFogComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();
	GetWorld()->Scene->RemoveExponentialHeightFog(uint64(this));
}

void UExponentialHeightFogComponent::OnUnregister()
{
	Super::OnUnregister();
	#if WITH_STATE_STREAM_ACTOR
	Handle = {};
	#endif
}

#if WITH_EDITOR

bool UExponentialHeightFogComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		static const auto CVarFog = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportExpFogMatchesVolumetricFog"));
		if (CVarFog && CVarFog->GetValueOnAnyThread() > 0)
		{
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, DirectionalInscatteringExponent) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, DirectionalInscatteringStartDistance) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, FogInscatteringLuminance) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, DirectionalInscatteringLuminance))
			{
				// In this case, all the data will come from the volumetric fog.
				return false;
			}
		}

		const bool bSupportsSkyLightCaptureAffectsHeightFogInScattering = SupportsSkyLightCaptureAffectsHeightFogInScattering(GMaxRHIShaderPlatform);
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, SkyLightCaptureAffectsHeightFogStrength))
		{
			return bSupportsSkyLightCaptureAffectsHeightFogInScattering;
		}
		const bool UsesSkyLightCaptureAffectsHeightFogInScattering = bSupportsSkyLightCaptureAffectsHeightFogInScattering && SkyLightCaptureAffectsHeightFogStrength > 0.0f;
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, SkyLightCaptureAffectsHeightFogRoughness))
		{
			return UsesSkyLightCaptureAffectsHeightFogInScattering;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, InscatteringColorCubemap))
		{
			return !UsesSkyLightCaptureAffectsHeightFogInScattering;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, DirectionalInscatteringExponent) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, DirectionalInscatteringStartDistance) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, DirectionalInscatteringLuminance) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, FogInscatteringLuminance))
		{
			return !InscatteringColorCubemap;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, FullyDirectionalInscatteringColorDistance) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, NonDirectionalInscatteringColorDistance) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, InscatteringTextureTint) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, InscatteringColorCubemapAngle))
		{
			return InscatteringColorCubemap != NULL && !UsesSkyLightCaptureAffectsHeightFogInScattering;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, FogInscatteringLuminance))
		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportSkyAtmosphereAffectsHeightFog"));
			return CVar && CVar->GetValueOnAnyThread() > 0;
		}
	}

	return Super::CanEditChange(InProperty);
}

void UExponentialHeightFogComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	SecondFogData.ClampToValidRanges();
	FogDensity = FMath::Clamp(FogDensity, 0.0f, 10.0f);
	FogHeightFalloff = FMath::Clamp(FogHeightFalloff, 0.0f, 2.0f);
	FogMaxOpacity = FMath::Clamp(FogMaxOpacity, 0.0f, 1.0f);
	StartDistance = FMath::Clamp(StartDistance, 0.0f, (float)WORLD_MAX);
	EndDistance = FMath::Clamp(EndDistance, 0.0f, (float)(10 * WORLD_MAX));
	FogCutoffDistance = FMath::Clamp(FogCutoffDistance, 0.0f, (float)(10 * WORLD_MAX));
	FullyDirectionalInscatteringColorDistance = FMath::Clamp(FullyDirectionalInscatteringColorDistance, 0.0f, (float)WORLD_MAX);
	NonDirectionalInscatteringColorDistance = FMath::Clamp(NonDirectionalInscatteringColorDistance, 0.0f, FullyDirectionalInscatteringColorDistance);
	InscatteringColorCubemapAngle = FMath::Clamp(InscatteringColorCubemapAngle, 0.0f, 360.0f);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

#if WITH_STATE_STREAM_ACTOR
	#define UEXPONENTIALHEIGHTFOG_SETHANDLE(Prefix, Field) \
		if (UseExponentialHeightFogStateStream) \
		{ \
			FExponentialHeightFogDynamicState Ds; \
			Ds.Set##Field(Prefix##Field); \
			Handle.Update(Ds); \
			return; \
		}
#else
	#define UEXPONENTIALHEIGHTFOG_SETHANDLE(Prefix, Field)
#endif


#define UEXPONENTIALHEIGHTFOG_VARIABLE(Type, Prefix, Name) \
	void UExponentialHeightFogComponent::Set##Name(Type NewValue) \
	{ \
		if (Name != NewValue) \
		{ \
			Name = NewValue; \
			UEXPONENTIALHEIGHTFOG_SETHANDLE(, Name) \
			MarkRenderStateDirty(); \
		} \
	}

UEXPONENTIALHEIGHTFOG_VARIABLES_WITH_SETFUNCTION
#undef UEXPONENTIALHEIGHTFOG_VARIABLE


void UExponentialHeightFogComponent::SetSecondFogDensity(float Value)
{
	if(SecondFogData.FogDensity != Value)
	{
		SecondFogData.FogDensity = Value;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(, SecondFogData)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetFogInscatteringColor(FLinearColor Value)
{
	if(FogInscatteringLuminance != Value)
	{
		FogInscatteringLuminance = Value;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(, FogInscatteringLuminance)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetInscatteringColorCubemap(UTextureCube* Value)
{
	if(InscatteringColorCubemap != Value)
	{
		InscatteringColorCubemap = Value;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(, InscatteringColorCubemap)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetFullyDirectionalInscatteringColorDistance(float Value)
{
	if(FullyDirectionalInscatteringColorDistance != Value)
	{
		FullyDirectionalInscatteringColorDistance = Value;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(, FullyDirectionalInscatteringColorDistance)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetDirectionalInscatteringColor(FLinearColor Value)
{
	if(DirectionalInscatteringLuminance != Value)
	{
		DirectionalInscatteringLuminance = Value;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(, DirectionalInscatteringLuminance)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetSecondFogHeightOffset(float Value)
{
	if(SecondFogData.FogHeightOffset != Value)
	{
		SecondFogData.FogHeightOffset = Value;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(, SecondFogData)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetSecondFogHeightFalloff(float Value)
{
	if(SecondFogData.FogHeightFalloff != Value)
	{
		SecondFogData.FogHeightFalloff = Value;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(, SecondFogData)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetVolumetricFog(bool bNewValue)
{
	if(bEnableVolumetricFog != bNewValue)
	{
		bEnableVolumetricFog = bNewValue;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(b, EnableVolumetricFog)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetSecondFogData(FExponentialHeightFogData NewValue)
{
	if(SecondFogData.FogDensity != NewValue.FogDensity ||
	   SecondFogData.FogHeightOffset != NewValue.FogHeightOffset ||
	   SecondFogData.FogHeightFalloff != NewValue.FogHeightFalloff)
	{
		SecondFogData = NewValue;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(, SecondFogData)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetHoldout(bool bNewHoldout)
{
	if (bHoldout != bNewHoldout)
	{
		bHoldout = bNewHoldout;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(b, Holdout)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::SetRenderInMainPass(bool bValue)
{
	if (bRenderInMainPass != bValue)
	{
		bRenderInMainPass = bValue;
		UEXPONENTIALHEIGHTFOG_SETHANDLE(b, RenderInMainPass)
		MarkRenderStateDirty();
	}
}

void UExponentialHeightFogComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.IsLoading() && (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::SkyAtmosphereAffectsHeightFogWithBetterDefault))
	{
		FogInscatteringLuminance = FogInscatteringColor_DEPRECATED;
		DirectionalInscatteringLuminance = DirectionalInscatteringColor_DEPRECATED;
	}
}

//////////////////////////////////////////////////////////////////////////
// AExponentialHeightFog

AExponentialHeightFog::AExponentialHeightFog(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Component = CreateDefaultSubobject<UExponentialHeightFogComponent>(TEXT("HeightFogComponent0"));
	RootComponent = Component;

	SetHidden(false);

#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet() && (GetSpriteComponent() != NULL))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> FogTextureObject;
			FName ID_Fog;
			FText NAME_Fog;
			FConstructorStatics()
				: FogTextureObject(TEXT("/Engine/EditorResources/S_ExpoHeightFog"))
				, ID_Fog(TEXT("Fog"))
				, NAME_Fog(NSLOCTEXT("SpriteCategory", "Fog", "Fog"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		GetSpriteComponent()->Sprite = ConstructorStatics.FogTextureObject.Get();
		GetSpriteComponent()->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
		GetSpriteComponent()->SpriteInfo.Category = ConstructorStatics.ID_Fog;
		GetSpriteComponent()->SpriteInfo.DisplayName = ConstructorStatics.NAME_Fog;
		GetSpriteComponent()->SetupAttachment(Component);
	}
#endif // WITH_EDITORONLY_DATA
}

void AExponentialHeightFog::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	bEnabled = Component->GetVisibleFlag();
}

void AExponentialHeightFog::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );
	
	DOREPLIFETIME( AExponentialHeightFog, bEnabled );
}

void AExponentialHeightFog::OnRep_bEnabled()
{
	Component->SetVisibility(bEnabled);
}

