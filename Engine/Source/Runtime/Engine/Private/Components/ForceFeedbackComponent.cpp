// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ForceFeedbackComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/World.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "Engine/Canvas.h"
#include "GenericPlatform/IInputInterface.h"
#include "Engine/Texture2D.h"
#include "UObject/ICookInfo.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ForceFeedbackComponent)

TArray<FForceFeedbackManager*> FForceFeedbackManager::PerWorldForceFeedbackManagers;
FDelegateHandle FForceFeedbackManager::OnWorldCleanupHandle;

#if WITH_EDITORONLY_DATA
static const TCHAR* GForceFeedbackSpriteAssetNameAutoActivate = TEXT("/Engine/EditorResources/S_ForceFeedbackComponent_AutoActivate.S_ForceFeedbackComponent_AutoActivate");
static const TCHAR* GForceFeedbackSpriteAssetName = TEXT("/Engine/EditorResources/S_ForceFeedbackComponent.S_ForceFeedbackComponent");
#endif

FForceFeedbackManager* FForceFeedbackManager::Get(UWorld* World, bool bCreateIfMissing)
{
	FForceFeedbackManager* ManagerForWorld = nullptr;
	
	for (FForceFeedbackManager* ForceFeedbackManager : PerWorldForceFeedbackManagers)
	{
		if (ForceFeedbackManager->World == World)
		{
			ManagerForWorld = ForceFeedbackManager;
			break;
		}
	}
	
	if (ManagerForWorld == nullptr && bCreateIfMissing)
	{
		if (!OnWorldCleanupHandle.IsValid())
		{
			OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddStatic(&FForceFeedbackManager::OnWorldCleanup);
		}

		ManagerForWorld = new FForceFeedbackManager(World);
		PerWorldForceFeedbackManagers.Add(ManagerForWorld);
	}

	return ManagerForWorld;
}

void FForceFeedbackManager::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	for (int32 Index = 0; Index < PerWorldForceFeedbackManagers.Num(); ++Index)
	{
		FForceFeedbackManager* ForceFeedbackManager = PerWorldForceFeedbackManagers[Index];
		if (ForceFeedbackManager->World == World)
		{
			delete ForceFeedbackManager;
			PerWorldForceFeedbackManagers.RemoveAtSwap(Index, EAllowShrinking::No);
			break;
		}
	}
}

void FForceFeedbackManager::AddActiveComponent(UForceFeedbackComponent* ForceFeedbackComponent)
{
	ActiveForceFeedbackComponents.AddUnique(ObjectPtrWrap(ForceFeedbackComponent));
}

void FForceFeedbackManager::RemoveActiveComponent(UForceFeedbackComponent* ForceFeedbackComponent)
{
	ActiveForceFeedbackComponents.RemoveSwap(ObjectPtrWrap(ForceFeedbackComponent));
}

void FForceFeedbackManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(World);
	Collector.AddReferencedObjects(ActiveForceFeedbackComponents);
}

FString FForceFeedbackManager::GetReferencerName() const
{
	return TEXT("FForceFeedbackManager");
}

UWorld* FForceFeedbackManager::GetTickableGameObjectWorld() const
{
	return World;
}

void FForceFeedbackManager::Tick(float DeltaTime)
{
	for (int32 Index = ActiveForceFeedbackComponents.Num() - 1; Index >= 0; --Index)
	{
		UForceFeedbackComponent* FFC = ActiveForceFeedbackComponents[Index];
		if (FFC)
		{
			if (!FFC->Advance(DeltaTime))
			{
				ActiveForceFeedbackComponents.RemoveAtSwap(Index, EAllowShrinking::No);
				FFC->StopInternal(false);
			}
		}
		else
		{
			ActiveForceFeedbackComponents.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}

	if (ActiveForceFeedbackComponents.Num() == 0)
	{
		PerWorldForceFeedbackManagers.RemoveSwap(this);
		delete this;
	}
}

void FForceFeedbackManager::Update(const FVector Location, FForceFeedbackValues& Values, const FPlatformUserId UserId) const
{
	for (UForceFeedbackComponent* FFC : ActiveForceFeedbackComponents)
	{
		if (FFC)
		{
			FFC->Update(Location, Values, UserId);
		}
	}
}

void FForceFeedbackManager::DrawDebug(const FVector Location, FDisplayDebugManager& DisplayDebugManager, const FPlatformUserId UserId) const
{
	for (UForceFeedbackComponent* FFC : ActiveForceFeedbackComponents)
	{
		if (FFC && FFC->ForceFeedbackEffect)
		{
			FForceFeedbackValues ActiveValues;
			FFC->Update(Location, ActiveValues, UserId);

			const FString ActiveEntry = FString::Printf(TEXT("%s %s %.2f %.2f %s %.2f - LL: %.2f LS: %.2f RL: %.2f RS: %.2f"), 
				*FFC->ForceFeedbackEffect->GetFName().ToString(), 
				*FFC->GetFName().ToString(),
				FVector::Dist(Location, FFC->GetComponentLocation()),
				FFC->ForceFeedbackEffect->GetDuration(),
				(FFC->bLooping ? TEXT("true") : TEXT("false")),
				FFC->PlayTime,
				ActiveValues.LeftLarge, ActiveValues.LeftSmall, ActiveValues.RightLarge, ActiveValues.RightSmall);

			DisplayDebugManager.DrawString(ActiveEntry);
		}
	}
}

bool FForceFeedbackManager::IsTickable() const
{
	return ActiveForceFeedbackComponents.Num() > 0;
}

TStatId FForceFeedbackManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FForceFeedbackManager, STATGROUP_Tickables);
}

UForceFeedbackComponent::UForceFeedbackComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bLooping = true;
	bAutoActivate = true;
	IntensityMultiplier = 1.f;

#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif
}

#if WITH_EDITORONLY_DATA
void UForceFeedbackComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UpdateSpriteTexture();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UForceFeedbackComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsSaving() && Ar.IsObjectReferenceCollector() && !Ar.IsCooking())
	{
		FSoftObjectPathSerializationScope EditorOnlyScope(ESoftObjectPathCollectType::EditorOnlyCollect);
		FSoftObjectPath SpriteAssets[]{ FSoftObjectPath(GForceFeedbackSpriteAssetNameAutoActivate), FSoftObjectPath(GForceFeedbackSpriteAssetName) };
		for (FSoftObjectPath& AssetPath : SpriteAssets)
		{
			Ar << AssetPath;
		}
	}
}


void UForceFeedbackComponent::UpdateSpriteTexture()
{
	if (SpriteComponent)
	{
		SpriteComponent->SpriteInfo.Category = TEXT("Misc");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Misc", "Misc");

		FCookLoadScope EditorOnlyScope(ECookLoadType::EditorOnly);
		if (bAutoActivate)
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(nullptr, GForceFeedbackSpriteAssetNameAutoActivate));
		}
		else
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(nullptr, GForceFeedbackSpriteAssetName));
		}
	}
}

void UForceFeedbackComponent::OnRegister()
{
	Super::OnRegister();

	UpdateSpriteTexture();
}
#endif

bool UForceFeedbackComponent::IsReadyForOwnerToAutoDestroy() const
{
	return !IsActive();
}

const UObject* UForceFeedbackComponent::AdditionalStatObject() const
{
	return ForceFeedbackEffect;
}

void UForceFeedbackComponent::OnUnregister()
{
	// Route OnUnregister event.
	Super::OnUnregister();

	// Don't stop feedback and clean up component if owner has been destroyed (default behaviour). This function gets
	// called from AActor::ClearComponents when an actor gets destroyed which is not usually what we want for one-
	// shot feedback effects.
	AActor* Owner = GetOwner();
	if (!Owner || bStopWhenOwnerDestroyed)
	{
		Stop();
	}
}

void UForceFeedbackComponent::Activate(bool bReset)
{
	if (bReset || ShouldActivate() == true)
	{
		Play();
	}

	Super::Activate(bReset);
}

void UForceFeedbackComponent::Deactivate()
{
	if (ShouldActivate() == false)
	{
		Stop();

		if (!IsActive())
		{
			OnComponentDeactivated.Broadcast(this);
		}
	}
}

void UForceFeedbackComponent::SetForceFeedbackEffect(UForceFeedbackEffect* NewForceFeedbackEffect)
{
	const bool bPlay = IsActive();

	// If this is an auto destroy component we need to prevent it from being auto-destroyed since we're really just restarting it
	const bool bWasAutoDestroy = bAutoDestroy;
	bAutoDestroy = false;
	Stop();
	bAutoDestroy = bWasAutoDestroy;

	ForceFeedbackEffect = NewForceFeedbackEffect;

	if (bPlay)
	{
		Play();
	}
}

void UForceFeedbackComponent::Play(const float StartTime)
{
	UWorld* World = GetWorld();

	if (IsActive())
	{
		// If this is an auto destroy component we need to prevent it from being auto-destroyed since we're really just restarting it
		const bool bCurrentAutoDestroy = bAutoDestroy;
		bAutoDestroy = false;
		Stop();
		bAutoDestroy = bCurrentAutoDestroy;
	}

	if (ForceFeedbackEffect && World)
	{
		SetActiveFlag(true);
		PlayTime = StartTime;
		FForceFeedbackManager::Get(World, true)->AddActiveComponent(this);
	}
}

void UForceFeedbackComponent::Stop()
{
	if (IsActive())
	{
		StopInternal(true);
	}
}

void UForceFeedbackComponent::StopInternal(const bool bRemoveFromManager)
{
	if (OnForceFeedbackFinished.IsBound())
	{
		OnForceFeedbackFinished.Broadcast(this);	
	}
	
	// Set this to immediately be inactive
	SetActiveFlag(false);
	PlayTime = 0.f;

	if (bRemoveFromManager)
	{
		if (FForceFeedbackManager* ForceFeedbackManager = FForceFeedbackManager::Get(GetWorld()))
		{
			ForceFeedbackManager->RemoveActiveComponent(this);
		}
	}

	// Auto destruction is handled via marking object for deletion.
	if (bAutoDestroy)
	{
		DestroyComponent();
	}
}

void UForceFeedbackComponent::SetIntensityMultiplier(const float NewIntensityMultiplier)
{
	IntensityMultiplier = NewIntensityMultiplier;
}

const FForceFeedbackAttenuationSettings* UForceFeedbackComponent::GetAttenuationSettingsToApply() const
{
	if (bOverrideAttenuation)
	{
		return &AttenuationOverrides;
	}
	else if (AttenuationSettings)
	{
		return &AttenuationSettings->Attenuation;
	}
	return nullptr;
}

void UForceFeedbackComponent::AdjustAttenuation(const FForceFeedbackAttenuationSettings& InAttenuationSettings)
{
	bOverrideAttenuation = true;
	AttenuationOverrides = InAttenuationSettings;
}

bool UForceFeedbackComponent::BP_GetAttenuationSettingsToApply(FForceFeedbackAttenuationSettings& OutAttenuationSettings) const
{
	if (const FForceFeedbackAttenuationSettings* Settings = GetAttenuationSettingsToApply())
	{
		OutAttenuationSettings = *Settings;
		return true;
	}
	return false;
}

void UForceFeedbackComponent::CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const
{
	if (const FForceFeedbackAttenuationSettings *AttenuationSettingsToApply = GetAttenuationSettingsToApply())
	{
		AttenuationSettingsToApply->CollectAttenuationShapesForVisualization(ShapeDetailsMap);
	}
}

bool UForceFeedbackComponent::Advance(const float DeltaTime)
{
	if (ForceFeedbackEffect == nullptr)
	{
		return false;
	}

	const float Duration = ForceFeedbackEffect->GetDuration();

	PlayTime += (bIgnoreTimeDilation ? FApp::GetDeltaTime() : DeltaTime);

	if (PlayTime > Duration && (!bLooping || Duration == 0.f) )
	{
		return false;
	}

	return true;
}

void UForceFeedbackComponent::Update(FVector Location, FForceFeedbackValues& Values, const FPlatformUserId UserId) const
{
	if (ForceFeedbackEffect)
	{
		const float Duration = ForceFeedbackEffect->GetDuration();
		const float EvalTime = PlayTime - Duration * FMath::FloorToFloat(PlayTime / Duration);

		float ValueMultiplier = IntensityMultiplier;

		if (ValueMultiplier > 0.f)
		{
			if (const FForceFeedbackAttenuationSettings* AttenuationSettingsToApply = GetAttenuationSettingsToApply())
			{
				ValueMultiplier *= AttenuationSettingsToApply->Evaluate(GetComponentTransform(), Location);
			}
		}

		if (ValueMultiplier > 0.f)
		{
			ForceFeedbackEffect->GetValues(EvalTime, Values, UserId, ValueMultiplier);
		}
	}
}

