// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/HeterogeneousVolumeComponent.h"

#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Stats/Stats.h"
#include "HeterogeneousVolumeInterface.h"
#include "LocalVertexFactory.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "StaticMeshResources.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HeterogeneousVolumeComponent)

DECLARE_CYCLE_STAT(TEXT("Get Dynamic Mesh Elements"), STAT_HeterogeneousVolumeComponentGetDynamicMeshElements, STATGROUP_HeterogeneousVolumesRT);
DECLARE_CYCLE_STAT(TEXT("Post Edit Change Property"), STAT_HeterogeneousVolumeComponentPostEditChangeProperty, STATGROUP_HeterogeneousVolumesGT);
DECLARE_CYCLE_STAT(TEXT("Component Tick"), STAT_HeterogeneousVolumeComponentTick, STATGROUP_HeterogeneousVolumesGT);

#define LOCTEXT_NAMESPACE "HeterogeneousVolumeComponent"

class FHeterogeneousVolumeSceneProxy : public FPrimitiveSceneProxy
{
public:
	FHeterogeneousVolumeSceneProxy(UHeterogeneousVolumeComponent* InComponent);
	virtual ~FHeterogeneousVolumeSceneProxy();

	//~ Begin FPrimitiveSceneProxy Interface.
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	virtual void GetDynamicMeshElements(
		const TArray<const FSceneView*>& Views,
		const FSceneViewFamily& ViewFamily,
		uint32 VisibilityMap,
		FMeshElementCollector& Collector) const override;

	virtual SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}
#if RHI_RAYTRACING
	virtual bool IsRayTracingRelevant() const override  { return true; }
	virtual bool HasRayTracingRepresentation() const override { return true; }
#endif // RHI_RAYTRACING
	virtual uint32 GetMemoryFootprint(void) const override { return sizeof(*this) + GetAllocatedSize(); }
	uint32 GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }
	//~ End FPrimitiveSceneProxy Interface.

private:
	FLocalVertexFactory VertexFactory;
	FStaticMeshVertexBuffers StaticMeshVertexBuffers;
	FHeterogeneousVolumeData HeterogeneousVolumeData;

	// Cache UObject values
	UMaterialInterface* MaterialInterface;
	FMaterialRelevance MaterialRelevance;
};

/*=============================================================================
	FHeterogeneousVolumeSceneProxy implementation.
=============================================================================*/

FHeterogeneousVolumeSceneProxy::FHeterogeneousVolumeSceneProxy(UHeterogeneousVolumeComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
	, VertexFactory(GetScene().GetFeatureLevel(), "FHeterogeneousVolumeSceneProxy")
#if ACTOR_HAS_LABELS
	, HeterogeneousVolumeData(this, InComponent->GetReadableName())
#else
	, HeterogeneousVolumeData(this)
#endif
	, MaterialInterface(InComponent->GetMaterial(0))
{
	bIsHeterogeneousVolume = true;
	bCastDynamicShadow = InComponent->CastShadow;

	// Heterogeneous volumes do not deform internally
	bHasDeformableMesh = false;
	ShadowCacheInvalidationBehavior = EShadowCacheInvalidationBehavior::Static;

	HeterogeneousVolumeData.VoxelResolution = InComponent->VolumeResolution;
	HeterogeneousVolumeData.InstanceToLocal = InComponent->FrameTransform.ToMatrixWithScale();

	// Infer minimum voxel size from bounds and resolution
	FVector VoxelSize = 2.0 * InComponent->Bounds.BoxExtent;
	VoxelSize.X /= InComponent->VolumeResolution.X;
	VoxelSize.Y /= InComponent->VolumeResolution.Y;
	VoxelSize.Z /= InComponent->VolumeResolution.Z;
	HeterogeneousVolumeData.MinimumVoxelSize = FMath::Max(VoxelSize.GetMin(), 0.001);

	if (InComponent->MaterialInstanceDynamic)
	{
		MaterialInterface = InComponent->MaterialInstanceDynamic;
	}

	if (MaterialInterface)
	{
		MaterialRelevance = MaterialInterface->GetRelevance_Concurrent(GetScene().GetShaderPlatform());
	}

	HeterogeneousVolumeData.StepFactor = InComponent->StepFactor;
	HeterogeneousVolumeData.ShadowStepFactor = InComponent->ShadowStepFactor;
	HeterogeneousVolumeData.ShadowBiasFactor = InComponent->ShadowBiasFactor;
	HeterogeneousVolumeData.LightingDownsampleFactor = InComponent->LightingDownsampleFactor;
	HeterogeneousVolumeData.MipBias = InComponent->StreamingMipBias;
	HeterogeneousVolumeData.bPivotAtCentroid = InComponent->bPivotAtCentroid;
	HeterogeneousVolumeData.bHoldout = InComponent->bHoldout;

	// Initialize vertex buffer data for a quad
	StaticMeshVertexBuffers.PositionVertexBuffer.Init(4);
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.Init(4, 1);
	StaticMeshVertexBuffers.ColorVertexBuffer.Init(4);

	for (uint32 VertexIndex = 0; VertexIndex < 4; ++VertexIndex)
	{
		StaticMeshVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = FColor::White;
	}

	StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(0) = FVector3f(-1.0, -1.0, -1.0);
	StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(1) = FVector3f(-1.0, 1.0, -1.0);
	StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(2) = FVector3f(1.0, -1.0, -1.0);
	StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(3) = FVector3f(1.0, 1.0, -1.0);

	StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(0, 0, FVector2f(0, 0));
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(1, 0, FVector2f(0, 1));
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(2, 0, FVector2f(1, 0));
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(3, 0, FVector2f(1, 1));

	FHeterogeneousVolumeSceneProxy* Self = this;
	ENQUEUE_RENDER_COMMAND(FHeterogeneousVolumeSceneProxyInit)(
		[Self](FRHICommandListImmediate& RHICmdList)
		{
			Self->StaticMeshVertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
			Self->StaticMeshVertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
			Self->StaticMeshVertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			Self->StaticMeshVertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&Self->VertexFactory, Data);
			Self->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&Self->VertexFactory, Data);
			Self->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&Self->VertexFactory, Data);
			Self->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(&Self->VertexFactory, Data, 0);
			Self->StaticMeshVertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&Self->VertexFactory, Data);
			Self->VertexFactory.SetData(RHICmdList, Data);

			Self->VertexFactory.InitResource(RHICmdList);
		}
	);
}

/** Virtual destructor. */
FHeterogeneousVolumeSceneProxy::~FHeterogeneousVolumeSceneProxy()
{
	VertexFactory.ReleaseResource();
	StaticMeshVertexBuffers.PositionVertexBuffer.ReleaseResource();
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
	StaticMeshVertexBuffers.ColorVertexBuffer.ReleaseResource();
}

FPrimitiveViewRelevance FHeterogeneousVolumeSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;

	MaterialRelevance.SetPrimitiveViewRelevance(Result);
	Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.HeterogeneousVolumes;
	Result.bOpaque = false;
	Result.bStaticRelevance = false;
	Result.bDynamicRelevance = true;
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bRenderInMainPass = ShouldRenderInMainPass();

	return Result;
}

void FHeterogeneousVolumeSceneProxy::GetDynamicMeshElements(
	const TArray<const FSceneView*>& Views,
	const FSceneViewFamily& ViewFamily,
	uint32 VisibilityMap,
	FMeshElementCollector& Collector) const
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	SCOPE_CYCLE_COUNTER(STAT_HeterogeneousVolumeComponentGetDynamicMeshElements);
	if (Views.IsEmpty())
	{
		return;
	}

	if (ViewFamily.EngineShowFlags.HeterogeneousVolumes)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				if (MaterialInterface)
				{
					// Set up MeshBatch
					FMeshBatch& Mesh = Collector.AllocateMesh();

					Mesh.VertexFactory = &VertexFactory;
					Mesh.MaterialRenderProxy = MaterialInterface->GetRenderProxy();
					Mesh.LCI = NULL;
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative() ? true : false;
					Mesh.CastShadow = CastsDynamicShadow();
					Mesh.Type = PT_TriangleStrip;
					Mesh.bDisableBackfaceCulling = true;

					// Set up the FMeshBatchElement.
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement.IndexBuffer = NULL;
					BatchElement.FirstIndex = 0;
					BatchElement.MinVertexIndex = 0;
					BatchElement.MaxVertexIndex = 3;
					BatchElement.NumPrimitives = 2;
					BatchElement.BaseVertexIndex = 0;

					// Heterogeneous Volume Interface is passed through UserData.
					BatchElement.UserData = &HeterogeneousVolumeData;

					Mesh.bCanApplyViewModeOverrides = true;
					Mesh.bUseWireframeSelectionColoring = IsSelected();
					Mesh.bUseSelectionOutline = false;
					Mesh.bSelectable = false;

					Collector.AddMesh(ViewIndex, Mesh);
				}

				const FSceneView* View = Views[ViewIndex];
				RenderBounds(Collector.GetPDI(ViewIndex), View->Family->EngineShowFlags, GetBounds(), IsSelected());
			}
		}
	}
}

/*=============================================================================
	HeterogeneousVolumeComponent implementation.
=============================================================================*/

UHeterogeneousVolumeComponent::UHeterogeneousVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	// What is this?
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;

#if WITH_EDITORONLY_DATA
	bTickInEditor = true;
#endif // WITH_EDITORONLY_DATA

	MaterialInstanceDynamic = nullptr;
	VolumeResolution = FIntVector(128);
	FrameRate = 24.0f;
	bPlaying = false;
	bLooping = false;
	Frame = 0;
	StartFrame = 0;
	EndFrame = 0;
	StepFactor = 1.0f;
	ShadowStepFactor = 2.0f;
	ShadowBiasFactor = 0.5f;
	LightingDownsampleFactor = 2.0f;
	StreamingMipBias = 0.0f;
	bIssueBlockingRequests = false;
	bPivotAtCentroid = false;
	PreviousSVT = nullptr;
}

void UHeterogeneousVolumeComponent::SetStreamingMipBias(int32 NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& StreamingMipBias != NewValue)
	{
		StreamingMipBias = NewValue;
		MarkRenderStateDirty();
	}
}

void UHeterogeneousVolumeComponent::SetVolumeResolution(FIntVector NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& VolumeResolution != NewValue)
	{
		VolumeResolution = NewValue;
		MarkRenderStateDirty();
	}
}

void UHeterogeneousVolumeComponent::SetFrame(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& Frame != NewValue)
	{
		Frame = NewValue;
		MarkRenderStateDirty();
	}
}

void UHeterogeneousVolumeComponent::SetFrameRate(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& FrameRate != NewValue)
	{
		FrameRate = NewValue;
		MarkRenderStateDirty();
	}
}

void UHeterogeneousVolumeComponent::SetStartFrame(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& StartFrame != NewValue)
	{
		StartFrame = NewValue;
		MarkRenderStateDirty();
	}
}

void UHeterogeneousVolumeComponent::SetEndFrame(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& EndFrame != NewValue)
	{
		EndFrame = NewValue;
		MarkRenderStateDirty();
	}
}

void UHeterogeneousVolumeComponent::SetPlaying(bool NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bPlaying != NewValue)
	{
		bPlaying = NewValue;
		MarkRenderStateDirty();
	}
}

void UHeterogeneousVolumeComponent::SetLooping(bool NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bLooping != NewValue)
	{
		bLooping = NewValue;
		MarkRenderStateDirty();
	}
}

void UHeterogeneousVolumeComponent::Play()
{
	if (AreDynamicDataChangesAllowed())
	{
		bPlaying = 1;
		Frame = 0;
		MarkRenderStateDirty();
	}
}

FPrimitiveSceneProxy* UHeterogeneousVolumeComponent::CreateSceneProxy()
{
	return new FHeterogeneousVolumeSceneProxy(this);
}

FBoxSphereBounds UHeterogeneousVolumeComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBoxSphereBounds NewBounds;

	FVector HalfVolumeResolution = FVector(VolumeResolution) * 0.5;
	if (bPivotAtCentroid)
	{
		NewBounds.Origin = FVector::ZeroVector;
	}
	else
	{
		NewBounds.Origin = HalfVolumeResolution;
	}
	NewBounds.BoxExtent = HalfVolumeResolution;
	NewBounds.SphereRadius = NewBounds.BoxExtent.Length();
	return NewBounds.TransformBy(FrameTransform * LocalToWorld);
}

void UHeterogeneousVolumeComponent::PostLoad()
{
	Super::PostLoad();

	MaterialInstanceDynamic = nullptr;
	if (UMaterialInterface* MaterialInterface = GetHeterogeneousVolumeMaterial())
	{
		MaterialInstanceDynamic = CreateOrCastToMID(MaterialInterface);
	}
}

void UHeterogeneousVolumeComponent::PostInitProperties()
{
	Super::PostInitProperties();

	if (IsInGameThread())
	{
		MaterialInstanceDynamic = nullptr;
		if (UMaterialInterface* MaterialInterface = GetHeterogeneousVolumeMaterial())
		{
			MaterialInstanceDynamic = CreateOrCastToMID(MaterialInterface);
		}
	}
}

void UHeterogeneousVolumeComponent::PostReinitProperties()
{
	Super::PostReinitProperties();

	if (IsInGameThread())
	{
		MaterialInstanceDynamic = nullptr;
		if (UMaterialInterface* MaterialInterface = GetHeterogeneousVolumeMaterial())
		{
			MaterialInstanceDynamic = CreateOrCastToMID(MaterialInterface);
		}
	}
}

USparseVolumeTexture* UHeterogeneousVolumeComponent::GetSparseVolumeTexture(UMaterialInterface* MaterialInterface, int32 ParameterIndex, FName* OutParamName)
{
	USparseVolumeTexture* SparseVolumeTexture = nullptr;

	if (MaterialInterface)
	{
		// Get parameter infos for all SVTs in the material
		TArray<FMaterialParameterInfo> ParameterInfo;
		TArray<FGuid> ParameterIds;
		MaterialInterface->GetAllSparseVolumeTextureParameterInfo(ParameterInfo, ParameterIds);

		// Get the SVT object
		if (ParameterInfo.IsValidIndex(ParameterIndex))
		{
			MaterialInterface->GetSparseVolumeTextureParameterValue(ParameterInfo[ParameterIndex], SparseVolumeTexture);
		}

		// The SVT in MaterialInterface might be a frame of a UStreamableSparseVolumeTexture. In that case we try to get the owning SVT object.
		if (SparseVolumeTexture && SparseVolumeTexture->IsA<USparseVolumeTextureFrame>())
		{
			USparseVolumeTextureFrame* Frame = Cast<USparseVolumeTextureFrame>(SparseVolumeTexture);
			UObject* FrameOuter = Frame->GetOuter();
			check(FrameOuter->IsA<UStreamableSparseVolumeTexture>());
			SparseVolumeTexture = Cast<USparseVolumeTexture>(FrameOuter);
			check(SparseVolumeTexture);
		}

		if (SparseVolumeTexture && OutParamName)
		{
			*OutParamName = ParameterInfo[ParameterIndex].Name;
		}
	}

	return SparseVolumeTexture;
}

UMaterialInstanceDynamic* UHeterogeneousVolumeComponent::CreateOrCastToMID(UMaterialInterface* MaterialInterface)
{
	if (MaterialInterface->IsA<UMaterialInstanceDynamic>())
	{
		return Cast<UMaterialInstanceDynamic>(MaterialInterface);
	}
	else
	{
		return UMaterialInstanceDynamic::Create(MaterialInterface, nullptr);
	}
}

void UHeterogeneousVolumeComponent::OnSparseVolumeTextureChanged(const USparseVolumeTexture* SparseVolumeTexture)
{
	if (SparseVolumeTexture != PreviousSVT)
	{
		if (SparseVolumeTexture)
		{
			VolumeResolution = SparseVolumeTexture->GetVolumeResolution();
			StartFrame = 0;
			EndFrame = SparseVolumeTexture->GetNumFrames() - 1;
			Frame = FMath::Clamp(Frame, StartFrame, EndFrame);
		}
		else
		{
			VolumeResolution = FIntVector(128);
			Frame = 0.0f;
			StartFrame = 0.0f;
			EndFrame = 0.0f;
		}

		PreviousSVT = SparseVolumeTexture;
		MarkRenderStateDirty();
	}
}

UMaterialInterface* UHeterogeneousVolumeComponent::GetHeterogeneousVolumeMaterial() const
{
	const uint32 MaterialIndex = 0;
	UMaterialInterface* MaterialInterface = GetMaterial(MaterialIndex);
	if (MaterialInterface)
	{
		const UMaterial* Material = MaterialInterface->GetMaterial();
		if (Material && Material->MaterialDomain == EMaterialDomain::MD_Volume)
		{
			MaterialInterface->CheckMaterialUsage(MATUSAGE_HeterogeneousVolumes);
			return MaterialInterface;
		}
	}
	return nullptr;
}

#if WITH_EDITOR
void UHeterogeneousVolumeComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	SCOPE_CYCLE_COUNTER(STAT_HeterogeneousVolumeComponentPostEditChangeProperty);
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const int32 SVTParameterIndex = 0;

	FName PropertyName;
	if (PropertyChangedEvent.Property)
	{
		PropertyName = PropertyChangedEvent.Property->GetFName();
	}

	// When this component is copied/duplicated in the editor, PostEditChangeProperty() is called with a null PropertyChangedEvent, so we also 
	// create the MID in that case. Otherwise the component will be copied but not play back because MaterialInstanceDynamic stays nullptr.
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, OverrideMaterials) || PropertyChangedEvent.Property == nullptr)
	{
		MaterialInstanceDynamic = nullptr; // Reset internal MID. We either create a new one from the new material or leave it as null if the material was unset
		UMaterialInterface* MaterialInterface = GetHeterogeneousVolumeMaterial();
		if (MaterialInterface)
		{
			MaterialInstanceDynamic = CreateOrCastToMID(MaterialInterface);
		}
		OnSparseVolumeTextureChanged(GetSparseVolumeTexture(MaterialInterface, SVTParameterIndex));
		MarkRenderStateDirty();
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, VolumeResolution))
	{
		// Prevent resolution changes when using SVT
		const USparseVolumeTexture* SparseVolumeTexture = GetSparseVolumeTexture(GetHeterogeneousVolumeMaterial(), SVTParameterIndex);
		if (SparseVolumeTexture)
		{
			VolumeResolution = SparseVolumeTexture->GetVolumeResolution();
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, Frame))
	{
		const USparseVolumeTexture* SparseVolumeTexture = GetSparseVolumeTexture(GetHeterogeneousVolumeMaterial(), SVTParameterIndex);
		if (SparseVolumeTexture)
		{
			Frame = FMath::Clamp(Frame, StartFrame, EndFrame);
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, StartFrame))
	{
		const USparseVolumeTexture* SparseVolumeTexture = GetSparseVolumeTexture(GetHeterogeneousVolumeMaterial(), SVTParameterIndex);
		if (SparseVolumeTexture)
		{
			StartFrame = FMath::Clamp(StartFrame, 0, EndFrame);
			Frame = FMath::Clamp(Frame, StartFrame, EndFrame);
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, EndFrame))
	{
		const USparseVolumeTexture* SparseVolumeTexture = GetSparseVolumeTexture(GetHeterogeneousVolumeMaterial(), SVTParameterIndex);
		if (SparseVolumeTexture)
		{
			const int32 FrameCount = SparseVolumeTexture->GetNumFrames();
			EndFrame = FMath::Clamp(EndFrame, StartFrame, FrameCount - 1);
			Frame = FMath::Clamp(Frame, StartFrame, EndFrame);
		}
	}
}
#endif // WITH_EDITOR

void UHeterogeneousVolumeComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	UMeshComponent::GetUsedMaterials(OutMaterials, bGetDebugMaterials);
	
	if (MaterialInstanceDynamic)
	{
		OutMaterials.Add(MaterialInstanceDynamic);
	}
}

bool UHeterogeneousVolumeComponent::IsMaterialSlotNameValid(FName MaterialSlotName) const
{
	return GetMaterialIndex(MaterialSlotName) >= 0;
}

int32 UHeterogeneousVolumeComponent::GetMaterialIndex(FName MaterialSlotName) const
{
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(Materials);
	for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
	{
		if (Materials[MaterialIndex]->GetFName() == MaterialSlotName)
		{
			return MaterialIndex;
		}
	}

	return INDEX_NONE;
}

void UHeterogeneousVolumeComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* Material)
{
	Super::SetMaterial(ElementIndex, Material);
	if (Material && ElementIndex == 0)
	{
		MaterialInstanceDynamic = nullptr; // Reset internal MID. We either create a new one from the new material or leave it as null if the material was unset
		UMaterialInterface* MaterialInterface = GetHeterogeneousVolumeMaterial();
		if (MaterialInterface)
		{
			MaterialInstanceDynamic = CreateOrCastToMID(MaterialInterface);
		}
		OnSparseVolumeTextureChanged(GetSparseVolumeTexture(MaterialInterface, 0 /*SVTParameterIndex*/));
	}
}

void UHeterogeneousVolumeComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	SCOPE_CYCLE_COUNTER(STAT_HeterogeneousVolumeComponentTick);
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// Forcibly update the MID if it's not initialized
	if (!MaterialInstanceDynamic)
	{
		UMaterialInterface* MaterialInterface = GetHeterogeneousVolumeMaterial();
		if (MaterialInterface)
		{
			MaterialInstanceDynamic = CreateOrCastToMID(MaterialInterface);
		}
	}

	// Aligning renderable state behavior with Instanced Static Mesh and Skinned Mesh. HV is currently not supporoted in indirect lighting and ray-tracing far field.
	if ((ShouldRender() || bCastHiddenShadow) && DoesRuntimeSupportHeterogeneousVolumes(GetWorld()->Scene->GetShaderPlatform()) && MaterialInstanceDynamic)
	{
		const int32 SVTParameterIndex = 0;
		FName SVTParameterName;
		USparseVolumeTexture* SparseVolumeTexture = GetSparseVolumeTexture(GetHeterogeneousVolumeMaterial(), SVTParameterIndex, &SVTParameterName);

#if WITH_EDITOR
		// Detect an update to the material
		if (SparseVolumeTexture != PreviousSVT)
		{
			OnSparseVolumeTextureChanged(SparseVolumeTexture);
		}
#endif // WITH_EDITOR

		if (SparseVolumeTexture)
		{
			const int32 FrameCount = SparseVolumeTexture->GetNumFrames();

			// Determine active frame based on animation controls if playing
			if (bPlaying)
			{
				Frame += DeltaTime * FrameRate;
			}

			if (bLooping)
			{
				float FrameRange = EndFrame - StartFrame + 1;
				Frame = FMath::Fmod(Frame - StartFrame, (float)FrameRange) + StartFrame;
			}
			else
			{
				Frame = FMath::Clamp(Frame, StartFrame, EndFrame);
			}

			const bool bIsBlocking = bIssueBlockingRequests != 0;
			const bool bHasValidFrameRate = bPlaying != 0;
			const float MipLevel = SparseVolumeTexture->GetOptimalStreamingMipLevel(Bounds, StreamingMipBias);
			USparseVolumeTextureFrame* SparseVolumeTextureFrame = USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SparseVolumeTexture, GetTypeHash(this), FrameRate, Frame, MipLevel, bIsBlocking, bHasValidFrameRate);
			if (SparseVolumeTextureFrame)
			{
				FIntVector PerFrameVolumeResolution = SparseVolumeTextureFrame->GetVolumeResolution();
				if (VolumeResolution != PerFrameVolumeResolution)
				{
					VolumeResolution = PerFrameVolumeResolution;
					MarkRenderTransformDirty();
				}

				FTransform PerFrameTransform = SparseVolumeTextureFrame->GetFrameTransform();
				if (!PerFrameTransform.Equals(FrameTransform))
				{
					FrameTransform = PerFrameTransform;
					MarkRenderStateDirty();
				}
			}

			MaterialInstanceDynamic->SetSparseVolumeTextureParameterValue(SVTParameterName, SparseVolumeTextureFrame);
		}
	}
}

AHeterogeneousVolume::AHeterogeneousVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	HeterogeneousVolumeComponent = CreateDefaultSubobject<UHeterogeneousVolumeComponent>(TEXT("HeterogeneousVolumeComponent"));
	RootComponent = HeterogeneousVolumeComponent;

#if WITH_EDITORONLY_DATA

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			FName ID_HeterogeneousVolume;
			FText NAME_HeterogeneousVolume;
			FConstructorStatics()
				: ID_HeterogeneousVolume(TEXT("Fog"))
				, NAME_HeterogeneousVolume(NSLOCTEXT("SpriteCategory", "Fog", "Fog"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;
		if (GetSpriteComponent())
		{
			GetSpriteComponent()->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			GetSpriteComponent()->bHiddenInGame = true;
			GetSpriteComponent()->bIsScreenSizeScaled = true;
			GetSpriteComponent()->SpriteInfo.Category = ConstructorStatics.ID_HeterogeneousVolume;
			GetSpriteComponent()->SpriteInfo.DisplayName = ConstructorStatics.NAME_HeterogeneousVolume;
			GetSpriteComponent()->SetupAttachment(HeterogeneousVolumeComponent);
			GetSpriteComponent()->bReceivesDecals = false;
		}
	}
#endif // WITH_EDITORONLY_DATA

	PrimaryActorTick.bCanEverTick = true;
	SetHidden(false);
}

#undef LOCTEXT_NAMESPACE