// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/InputComponent.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerInput.h"
#include "GameFramework/InputSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputComponent)


void FInputActionBinding::GenerateNewHandle()
{
	// Must be in C++ to avoid duplicate statics accross execution units
	static int32 GHandle = 1;
	Handle = GHandle++;
}

/* UInputComponent interface
 *****************************************************************************/

UInputComponent::UInputComponent( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	bBlockInput = false;
}

void UInputComponent::ConditionalBuildKeyMap(UPlayerInput* PlayerInput)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInputComponent::ConditionalBuildKeyMap);
	
	if (!ensure(PlayerInput))
	{
		return;
	}

	FCachedKeyToActionInfo* CachedInfoToPopulate = nullptr;

	for (int32 Index = CachedKeyToActionInfo.Num() - 1; Index >= 0; --Index)
	{
		CachedInfoToPopulate = &CachedKeyToActionInfo[Index];

		if (CachedInfoToPopulate->PlayerInput == PlayerInput)
		{
			if (CachedInfoToPopulate->KeyMapBuiltForIndex == PlayerInput->GetKeyMapBuildIndex())
			{
				// Found it and it doesn't need to be rebuilt
				return;
			}
			// Found it and it does need to be rebuilt
			break;
		}
		else if (CachedInfoToPopulate->PlayerInput == nullptr)
		{
			CachedKeyToActionInfo.RemoveAtSwap(Index, EAllowShrinking::No);
		}

		CachedInfoToPopulate = nullptr;
	}

	if (CachedInfoToPopulate == nullptr)
	{
		CachedKeyToActionInfo.AddDefaulted();
		CachedInfoToPopulate = &CachedKeyToActionInfo.Last();
		CachedInfoToPopulate->PlayerInput = PlayerInput;
		if (AActor* OwnerActor = PlayerInput->GetTypedOuter<AActor>())
		{
			OwnerActor->OnEndPlay.AddUniqueDynamic(this, &UInputComponent::OnInputOwnerEndPlayed);
		}
	}

	// Reset the map and AnyKey array
	for (TPair<FKey, TArray<TSharedPtr<FInputActionBinding>>>& KeyBindingPair : CachedInfoToPopulate->KeyToActionMap)
	{
		KeyBindingPair.Value.Reset();
	}
	CachedInfoToPopulate->AnyKeyToActionMap.Reset();

	for (const TSharedPtr<FInputActionBinding>& ActionBinding : ActionBindings)
	{
		const TArray<FInputActionKeyMapping>& KeysForAction = PlayerInput->GetKeysForAction(ActionBinding->ActionName);

		for (const FInputActionKeyMapping& KeyMapping : KeysForAction)
		{
			if (KeyMapping.Key != EKeys::AnyKey)
			{
				CachedInfoToPopulate->KeyToActionMap.FindOrAdd(KeyMapping.Key).Add(ActionBinding);
			}
			else
			{
				CachedInfoToPopulate->AnyKeyToActionMap.Add(ActionBinding);
			}
		}
	}

	CachedInfoToPopulate->KeyMapBuiltForIndex = PlayerInput->GetKeyMapBuildIndex();
}

void UInputComponent::OnInputOwnerEndPlayed(AActor* InOwner, EEndPlayReason::Type EndPlayReason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInputComponent::OnInputOwnerEndPlayed);
	
	for (int32 Index = CachedKeyToActionInfo.Num() - 1; Index >= 0; --Index)
	{
		FCachedKeyToActionInfo& CachedInfo = CachedKeyToActionInfo[Index];
		const UPlayerInput* CachedInput = CachedInfo.PlayerInput.Get();
		if (CachedInput && CachedInput->GetTypedOuter<AActor>() == InOwner)
		{
			CachedKeyToActionInfo.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}
}

void UInputComponent::ClearBindingsForObject(UObject* InOwner)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInputComponent::ClearBindingsForObject);
	
	for (int32 Index = CachedKeyToActionInfo.Num() - 1; Index >= 0; --Index)
	{
		FCachedKeyToActionInfo& CachedInfo = CachedKeyToActionInfo[Index];
		const UPlayerInput* CachedInput = CachedInfo.PlayerInput.Get();
		if (CachedInput && CachedInput->GetTypedOuter<UObject>() == InOwner)
		{
			CachedKeyToActionInfo.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}
}

void UInputComponent::LogDeprecatedBindingWarning(const FName BindingName)
{
	const UInputSettings* Settings = GetDefault<UInputSettings>();
	if (Settings->bEnabledLegacyMappingDeprecationWarnings && Settings->IsActionMappingDeprecated(BindingName))
	{
		UE_LOGF(LogPlayerInput, Warning, "Attempting to access a deprecated action/axis mapping '%ls'", *BindingName.ToString());
	}
}

void UInputComponent::GetActionsBoundToKey(UPlayerInput* PlayerInput, const FKey Key, TArray<TSharedPtr<FInputActionBinding>>& Actions) const
{
	for (const FCachedKeyToActionInfo& CachedInfo : CachedKeyToActionInfo)
	{
		if (CachedInfo.PlayerInput == PlayerInput)
		{
			if (const TArray<TSharedPtr<FInputActionBinding>>* ActionsForKey = CachedInfo.KeyToActionMap.Find(Key))
			{
				for (const TSharedPtr<FInputActionBinding>& ActionForKey : *ActionsForKey)
				{
					Actions.AddUnique(ActionForKey);
				}
			}
			for (const TSharedPtr<FInputActionBinding>& ActionForKey : CachedInfo.AnyKeyToActionMap)
			{
				Actions.AddUnique(ActionForKey);
			}
			return;
		}
	}
	// If we get here then we failed to find cached actions for the specified PlayerInput which means the conditional function has not yet been called
	// We'll ensure so high level code is fixed, and then force it to be built and recall ourselves
	ensure(false);
	UInputComponent* MutableThis = const_cast<UInputComponent*>(this);
	MutableThis->ConditionalBuildKeyMap(PlayerInput);
	GetActionsBoundToKey(PlayerInput, Key, Actions);
}

float UInputComponent::GetAxisValue( const FName AxisName ) const
{
	float AxisValue = 0.f;
	if (AxisName.IsNone())
	{
		return AxisValue;
	}

	bool bFound = false;

	for (const FInputAxisBinding& AxisBinding : AxisBindings)
	{
		if (AxisBinding.AxisName == AxisName)
		{
			bFound = true;
			AxisValue = AxisBinding.AxisValue;
			break;
		}
	}

	if (!bFound)
	{
		UE_LOGF(LogPlayerController, Warning, "Request for value of axis '%ls' returning 0 as it is not bound on this input component.", *AxisName.ToString());
	}

	return AxisValue;
}


float UInputComponent::GetAxisKeyValue( const FKey AxisKey ) const
{
	float AxisValue = 0.f;
	bool bFound = false;

	for (const FInputAxisKeyBinding& AxisBinding : AxisKeyBindings)
	{
		if (AxisBinding.AxisKey == AxisKey)
		{
			bFound = true;
			AxisValue = AxisBinding.AxisValue;
			break;
		}
	}

	if (!bFound)
	{
		UE_LOGF(LogPlayerController, Warning, "Request for value of axis key '%ls' returning 0 as it is not bound on this input component.", *AxisKey.ToString());
	}

	return AxisValue;
}


FVector UInputComponent::GetVectorAxisValue( const FKey AxisKey ) const
{
	FVector AxisValue = FVector::ZeroVector;
	bool bFound = false;

	for (const FInputVectorAxisBinding& AxisBinding : VectorAxisBindings)
	{
		if (AxisBinding.AxisKey == AxisKey)
		{
			AxisValue = AxisBinding.AxisValue;
			bFound = true;
			break;
		}
	}

	if (!bFound)
	{
		UE_LOGF(LogPlayerController, Warning, "Request for value of vector axis key '%ls' returning 0 as it is not bound on this input component.", *AxisKey.ToString());
	}

	return AxisValue;
}


bool UInputComponent::HasBindings( ) const
{
	return ((ActionBindings.Num() > 0) ||
			(AxisBindings.Num() > 0) ||
			(AxisKeyBindings.Num() > 0) ||
			(KeyBindings.Num() > 0) ||
			(TouchBindings.Num() > 0) ||
			(GestureBindings.Num() > 0) ||
			(VectorAxisBindings.Num() > 0));
}


FInputActionBinding& UInputComponent::AddActionBinding( FInputActionBinding InBinding )
{
	LogDeprecatedBindingWarning(InBinding.GetActionName());
	
	ActionBindings.Add(MakeShared<FInputActionBinding>(MoveTemp(InBinding)));
	FInputActionBinding& Binding = *ActionBindings.Last().Get();
	Binding.GenerateNewHandle();

	if (Binding.KeyEvent == IE_Pressed || Binding.KeyEvent == IE_Released)
	{
		const EInputEvent PairedEvent = (Binding.KeyEvent == IE_Pressed ? IE_Released : IE_Pressed);
		for (int32 BindingIndex = ActionBindings.Num() - 2; BindingIndex >= 0; --BindingIndex )
		{
			FInputActionBinding& ActionBinding = *ActionBindings[BindingIndex].Get();
			if (ActionBinding.ActionName == Binding.ActionName)
			{
				// If we find a matching event that is already paired we know this is paired so mark it off and we're done
				if (ActionBinding.bPaired)
				{
					Binding.bPaired = true;
					break;
				}
				// Otherwise if this is a pair to the new one mark them both as paired
				// Don't break as there could be two bound paired events
				else if (ActionBinding.KeyEvent == PairedEvent)
				{
					ActionBinding.bPaired = true;
					Binding.bPaired = true;
				}
			}
		}
	}

	for (FCachedKeyToActionInfo& CachedInfo : CachedKeyToActionInfo)
	{
		CachedInfo.KeyMapBuiltForIndex = 0;
	}

	return Binding;
}


void UInputComponent::ClearActionBindings( )
{
	for (FCachedKeyToActionInfo& CachedInfo : CachedKeyToActionInfo)
	{
		CachedInfo.KeyMapBuiltForIndex = 0;
	}
	ActionBindings.Reset();
}

void UInputComponent::RemoveActionBinding( const int32 BindingIndex )
{
	if (BindingIndex >= 0 && BindingIndex < ActionBindings.Num())
	{
		const FInputActionBinding& BindingToRemove = *ActionBindings[BindingIndex].Get();
		RemoveActionBinding(BindingToRemove, BindingIndex);
	}
}

void UInputComponent::RemoveActionBindingForHandle(const int32 Handle)
{
	for (int32 ActionIndex = 0; ActionIndex < ActionBindings.Num(); ++ActionIndex)
	{
		const FInputActionBinding& BindingToRemove = *ActionBindings[ActionIndex].Get();
		if (BindingToRemove.GetHandle() == Handle)
		{
			RemoveActionBinding(BindingToRemove, ActionIndex);
			return;
		}
	}
}

void UInputComponent::RemoveActionBinding(const FInputActionBinding &BindingToRemove, const int32 BindingIndex)
{
	LogDeprecatedBindingWarning(BindingToRemove.GetActionName());
	
	// Potentially need to clear some pairings
	if (BindingToRemove.bPaired)
	{
		TArray<int32> IndicesToClear;
		const EInputEvent PairedEvent = (BindingToRemove.KeyEvent == IE_Pressed ? IE_Released : IE_Pressed);

		for (int32 ActionIndex = 0; ActionIndex < ActionBindings.Num(); ++ActionIndex)
		{
			if (ActionIndex != BindingIndex)
			{
				const FInputActionBinding& ActionBinding = *ActionBindings[ActionIndex].Get();
				if (ActionBinding.ActionName == BindingToRemove.ActionName)
				{
					// If we find another of the same key event then the pairing is intact so we're done
					if (ActionBinding.KeyEvent == BindingToRemove.KeyEvent)
					{
						IndicesToClear.Empty();
						break;
					}
					// Otherwise we may need to clear the pairing so track the index
					else if (ActionBinding.KeyEvent == PairedEvent)
					{
						IndicesToClear.Add(ActionIndex);
					}
				}
			}
		}

		for (int32 ClearIndex = 0; ClearIndex < IndicesToClear.Num(); ++ClearIndex)
		{
			ActionBindings[IndicesToClear[ClearIndex]]->bPaired = false;
		}
	}

	ActionBindings.RemoveAt(BindingIndex, EAllowShrinking::No);
	for (FCachedKeyToActionInfo& CachedInfo : CachedKeyToActionInfo)
	{
		CachedInfo.KeyMapBuiltForIndex = 0;
	}
}

void UInputComponent::RemoveActionBinding(FName InActionName, EInputEvent InKeyEvent)
{
	LogDeprecatedBindingWarning(InActionName);
	
	for (int32 ActionIdx = 0; ActionIdx < ActionBindings.Num(); ++ActionIdx)
	{
		const TSharedPtr<FInputActionBinding>& Binding = ActionBindings[ActionIdx];
		if (Binding->GetActionName() == InActionName && Binding->KeyEvent == InKeyEvent)
		{
			RemoveActionBinding(ActionIdx);
			break;
		}
	}
}

void UInputComponent::ClearBindingValues()
{
	for (FInputAxisBinding& AxisBinding : AxisBindings)
	{
		AxisBinding.AxisValue = 0.f;
	}
	for (FInputAxisKeyBinding& AxisKeyBinding : AxisKeyBindings)
	{
		AxisKeyBinding.AxisValue = 0.f;
	}
	for (FInputVectorAxisBinding& VectorAxisBinding : VectorAxisBindings)
	{
		VectorAxisBinding.AxisValue = FVector::ZeroVector;
	}
	for (FInputGestureBinding& GestureBinding : GestureBindings)
	{
		GestureBinding.GestureValue = 0.f;
	}
}

void UInputComponent::RemoveAxisBinding(FName AxisName)
{
	LogDeprecatedBindingWarning(AxisName);
	
	for (int32 AxisIdx = AxisBindings.Num() - 1; AxisIdx >= 0; --AxisIdx)
	{
		const FInputAxisBinding& Binding = AxisBindings[AxisIdx];
		if (Binding.AxisName == AxisName)
		{
			AxisBindings.RemoveAt(AxisIdx, EAllowShrinking::No);
		}
	}
}

void UInputComponent::ClearAxisBindings()
{
	AxisBindings.Reset();
}

void UInputComponent::RemoveAxisKeyBinding(FKey AxisKey)
{
	for (int32 AxisIdx = AxisKeyBindings.Num() - 1; AxisIdx >= 0; --AxisIdx)
	{
		const FInputAxisKeyBinding& Binding = AxisKeyBindings[AxisIdx];
		if (Binding.AxisKey == AxisKey)
		{
			AxisKeyBindings.RemoveAt(AxisIdx, EAllowShrinking::No);
		}
	}
}

/* Deprecated functions (needed for Blueprints)
 *****************************************************************************/

bool UInputComponent::IsControllerKeyDown(FKey Key) const { return false; }
bool UInputComponent::WasControllerKeyJustPressed(FKey Key) const { return false; }
bool UInputComponent::WasControllerKeyJustReleased(FKey Key) const { return false; }
float UInputComponent::GetControllerAnalogKeyState(FKey Key) const { return 0.f; }
FVector UInputComponent::GetControllerVectorKeyState(FKey Key) const { return FVector(); }
void UInputComponent::GetTouchState(int32 FingerIndex, float& LocationX, float& LocationY, bool& bIsCurrentlyPressed) const { }
float UInputComponent::GetControllerKeyTimeDown(FKey Key) const { return 0.f; }
void UInputComponent::GetControllerMouseDelta(float& DeltaX, float& DeltaY) const { }
void UInputComponent::GetControllerAnalogStickState(EControllerAnalogStick::Type WhichStick, float& StickX, float& StickY) const { }

