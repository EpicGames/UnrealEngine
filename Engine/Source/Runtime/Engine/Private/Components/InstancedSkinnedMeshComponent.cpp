// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/InstancedSkinnedMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Elements/SMInstance/SMInstanceElementData.h" // For SMInstanceElementDataUtil::SMInstanceElementsEnabled
#include "Elements/SMInstance/SMInstanceElementId.h"
#include "Engine/OverlapResult.h"
#include "Engine/StaticMesh.h"
#include "HitProxies.h"
#include "NaniteSceneProxy.h"
#include "PrimitiveSceneInfo.h"
#include "Engine/SkeletalMeshSocket.h"
#include "SkinningSceneExtensionProxy.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Rendering/NaniteResourcesHelper.h"
#include "Rendering/RenderCommandPipes.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SceneInterface.h"
#include "SkeletalRenderNanite.h"
#include "SkeletalRenderGPUSkin.h"
#include "SkinningDefinitions.h"
#include "SkinnedMeshSceneProxyDesc.h"
#include "InstanceData/InstanceUpdateChangeSet.h"
#include "InstanceData/InstanceDataUpdateUtils.h"
#include "InstancedSkinnedMeshComponentHelper.h"
#include "PrimitiveSceneDesc.h"
#include "InstancedSkinnedMeshSceneProxy.h"
#include "InstancedMeshComponentBodies.h"
#include "AnimationRuntime.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "WorldCollision.h"
#include "PhysicsEngine/BodySetup.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Physics/Experimental/PhysScene_Chaos.h"

#if WITH_EDITOR
#include "WorldPartition/HLOD/HLODHashBuilder.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedSkinnedMeshComponent)

DEFINE_LOG_CATEGORY_STATIC(LogInstancedSkinnedMesh, Log, All);

TAutoConsoleVariable<int32> CVarInstancedSkinnedMeshesForceRefPose(
	TEXT("r.InstancedSkinnedMeshes.ForceRefPose"),
	0,
	TEXT("Whether to force ref pose for instanced skinned meshes"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarInstancedSkinnedMeshesAnimationBounds(
	TEXT("r.InstancedSkinnedMeshes.AnimationBounds"),
	1,
	TEXT("Whether to use animation bounds for instanced skinned meshes"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
);

static FSkeletalMeshObject* CreateInstancedSkeletalMeshObjectFunction(void* UserData, USkinnedMeshComponent* InComponent, FSkeletalMeshRenderData* InRenderData, ERHIFeatureLevel::Type InFeatureLevel)
{
	return FInstancedSkinnedMeshSceneProxyDesc::CreateMeshObject(FInstancedSkinnedMeshSceneProxyDesc(Cast<UInstancedSkinnedMeshComponent>(InComponent)), InRenderData, InFeatureLevel);
}

UInstancedSkinnedMeshComponent::UInstancedSkinnedMeshComponent(FVTableHelper& Helper)
: Super(Helper)
, bInheritPerInstanceData(false)
, bUseGpuLodSelection(true)
, bDisableCollision(false)
, InstancePhysicsBodies(MakeUnique<FInstancedMeshComponentBodies>(this))
, InstanceDataManager(this)
{
}

UInstancedSkinnedMeshComponent::UInstancedSkinnedMeshComponent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
, bInheritPerInstanceData(false)
, bUseGpuLodSelection(true)
, bDisableCollision(false)
, InstancePhysicsBodies(MakeUnique<FInstancedMeshComponentBodies>(this))
, InstanceDataManager(this)
{
}

UInstancedSkinnedMeshComponent::~UInstancedSkinnedMeshComponent()
{
}

bool UInstancedSkinnedMeshComponent::ShouldForceRefPose()
{ 
	return CVarInstancedSkinnedMeshesForceRefPose.GetValueOnAnyThread() != 0;
}

bool UInstancedSkinnedMeshComponent::ShouldUseAnimationBounds()
{
	return CVarInstancedSkinnedMeshesAnimationBounds.GetValueOnAnyThread() != 0;
}

struct FSkinnedMeshInstanceData_Deprecated
{
	FMatrix Transform;
	uint32 AnimationIndex;
	uint32 Padding[3]; // Need to respect 16 byte alignment for bulk-serialization

	FSkinnedMeshInstanceData_Deprecated()
	: Transform(FMatrix::Identity)
	, AnimationIndex(0)
	{
		Padding[0] = 0;
		Padding[1] = 0;
		Padding[2] = 0;
	}

	FSkinnedMeshInstanceData_Deprecated(const FMatrix& InTransform, uint32 InAnimationIndex)
	: Transform(InTransform)
	, AnimationIndex(InAnimationIndex)
	{
		Padding[0] = 0;
		Padding[1] = 0;
		Padding[2] = 0;
	}

	friend FArchive& operator<<(FArchive& Ar, FSkinnedMeshInstanceData_Deprecated& InstanceData)
	{
		// @warning BulkSerialize: FSkinnedMeshInstanceData is serialized as memory dump
		// See TArray::BulkSerialize for detailed description of implied limitations.
		Ar << InstanceData.Transform;
		Ar << InstanceData.AnimationIndex;
		Ar << InstanceData.Padding[0];
		Ar << InstanceData.Padding[1];
		Ar << InstanceData.Padding[2];
		return Ar;
	}
};

void UInstancedSkinnedMeshComponent::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	// Inherit properties when bEditableWhenInherited == false || bInheritPerInstanceData == true (when the component isn't a template and we are persisting data)
	const UInstancedSkinnedMeshComponent* Archetype = Cast<UInstancedSkinnedMeshComponent>(GetArchetype());
	const bool bInheritSkipSerializationProperties = ShouldInheritPerInstanceData(Archetype) && Ar.IsPersistent();
	
	// Check if we need have SkipSerialization property data to load/save
	bool bHasSkipSerializationPropertiesData = !bInheritSkipSerializationProperties;
	Ar << bHasSkipSerializationPropertiesData;

	if (Ar.IsLoading())
	{
		// Read existing data if it was serialized
		TArray<FSkinnedMeshInstanceData> TempInstanceData;
		TArray<float> TempInstanceCustomData;

		if (bHasSkipSerializationPropertiesData)
		{
			if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::SkinnedMeshInstanceDataSerializationV2)
			{
				TArray<FSkinnedMeshInstanceData_Deprecated> TempInstanceData_Deprecated;
				TempInstanceData_Deprecated.BulkSerialize(Ar, false /* force per element serialization */);

				TempInstanceData.Reserve(TempInstanceData_Deprecated.Num());
				for (const auto& Item : TempInstanceData_Deprecated)
				{
					TempInstanceData.Emplace(FTransform3f(FMatrix44f(Item.Transform)), Item.AnimationIndex);
				}
			}
			else
			{
				Ar << TempInstanceData;
			}
			TempInstanceCustomData.BulkSerialize(Ar);
		}

		// If we should inherit use Archetype Data
		if (bInheritSkipSerializationProperties)
		{
			ApplyInheritedPerInstanceData(Archetype);
		} 
		// It is possible for a component to lose its BP archetype between a save / load so in this case we have no per instance data (usually this component gets deleted through construction script)
		else if (bHasSkipSerializationPropertiesData)
		{
			InstanceData = MoveTemp(TempInstanceData);
			InstanceCustomData = MoveTemp(TempInstanceCustomData);
		}
	}
	else if (bHasSkipSerializationPropertiesData)
	{
		Ar << InstanceData;
		InstanceCustomData.BulkSerialize(Ar);
	}

#if WITH_EDITOR
	if (Ar.IsTransacting())
	{
		Ar << SelectedInstances;
	}
#endif

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SkinnedMeshInstanceDataSerializationV2)
	{
		InstanceDataManager.Serialize(Ar, bCooked);
	}
	else if (Ar.IsLoading())
	{
		// Prior to this the id mapping was not saved so we need to reset it.
		InstanceDataManager.Reset(InstanceData.Num());
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::InstancedSkinnedMeshBoneAttachments)
	{
		if (Ar.IsSaving())
		{
			SaveBoneAttachments(Ar);
		}
		else
		{
			LoadBoneAttachments(Ar);
		}
	}
	else if (Ar.IsLoading())
	{
		// Old asset without bone attachment data. Initialize parallel arrays to match instance count.
		BoneAttachmentBindings.SetNum(InstanceData.Num());
		DirtyBoneAttachmentBindings.Init(false, InstanceData.Num());
		NumDirtyBoneAttachmentBindings = 0;
	}

	if (bCooked)
	{
		if (Ar.IsLoading())
		{
			InstanceDataManager.ReadCookedRenderData(Ar);
		}
#if WITH_EDITOR
		else if (Ar.IsSaving())
		{
			InstanceDataManager.WriteCookedRenderData(Ar, GetComponentDesc(GMaxRHIShaderPlatform));
		}
#endif
	}
}

void UInstancedSkinnedMeshComponent::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GIsEditor)
	{
		SetSkinnedAssetCallback();
	}
#endif
}

void UInstancedSkinnedMeshComponent::BeginDestroy()
{
	Super::BeginDestroy();
}

void UInstancedSkinnedMeshComponent::OnRegister()
{
	Super::OnRegister();

	// Propagate the current number of instances to the manager. Because of the various ways
	// instance data can be populated (property copy from archetype, ApplyInheritedPerInstanceData,
	// etc.) the manager's count can get out of sync with InstanceData. Reset if mismatched.
	// Skip for GPU-only mode: InstanceData is empty and the manager is driven by GPU-side data.
	if (!bIsInstanceDataGPUOnly && InstanceDataManager.GetMaxInstanceIndex() != InstanceData.Num())
	{
		InstanceDataManager.Reset(InstanceData.Num());
		InstanceDataManager.ClearChangeTracking();
	}
}

void UInstancedSkinnedMeshComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}

bool UInstancedSkinnedMeshComponent::IsEnabled() const
{
	return FInstancedSkinnedMeshComponentHelper::IsEnabled(*this);
}

int32 UInstancedSkinnedMeshComponent::GetInstanceCount() const
{
	return bIsInstanceDataGPUOnly ? NumInstancesGPUOnly : InstanceData.Num();
}

FPrimitiveInstanceId UInstancedSkinnedMeshComponent::GetInstanceId(int32 InstanceIndex) const
{
	if (!bIsInstanceDataGPUOnly && InstanceIndex >= 0 && InstanceIndex < InstanceDataManager.GetMaxInstanceIndex())
	{
		return InstanceDataManager.IndexToId(InstanceIndex);
	}
	return FPrimitiveInstanceId();
}

UTransformProviderData* UInstancedSkinnedMeshComponent::GetTransformProvider() const
{
	return TransformProvider.Get();
}

void UInstancedSkinnedMeshComponent::SetTransformProvider(UTransformProviderData* InTransformProvider)
{
	TransformProvider = InTransformProvider;
	// We use the transform dirty state to drive the update of the animation data (to defer the need to add more bits), so we mark those as dirty here.
	InstanceDataManager.TransformsChangedAll();
	MarkRenderStateDirty();
}

template <typename ArrayType>
inline void ReorderArray(ArrayType& InOutDataArray, const TArray<int32>& OldIndexArray, int32 ElementStride = 1)
{
	check(OldIndexArray.Num() * ElementStride == InOutDataArray.Num());
	ArrayType TmpDataArray = MoveTemp(InOutDataArray);
	InOutDataArray.Empty(TmpDataArray.Num());
	for (int32 NewIndex = 0; NewIndex < OldIndexArray.Num(); ++NewIndex)
	{
		int32 OldIndex = OldIndexArray[NewIndex];
		for (int32 SubIndex = 0; SubIndex < ElementStride; ++SubIndex)
		{
			InOutDataArray.Add(TmpDataArray[OldIndex * ElementStride + SubIndex]);
		}
	}
}

void UInstancedSkinnedMeshComponent::OptimizeInstanceData(bool bShouldRetainIdMap)
{
	// compute the optimal order 
	TArray<int32> IndexRemap = InstanceDataManager.Optimize(GetComponentDesc(GMaxRHIShaderPlatform), bShouldRetainIdMap);
	
	if (!IndexRemap.IsEmpty())
	{
		// Reorder instances according to the remap
		ReorderArray(InstanceData, IndexRemap);
		ReorderArray(InstanceCustomData, IndexRemap, NumCustomDataFloats);
		if (bHasPreviousTransforms && PerInstancePrevTransform.Num() == InstanceData.Num())
		{
			ReorderArray(PerInstancePrevTransform, IndexRemap);
		}
#if WITH_EDITOR
		if (SelectedInstances.Num() == InstanceData.Num())
		{
			ReorderArray(SelectedInstances, IndexRemap);
		}
		else
		{
			// Force to correct length and zero fill. Safe because nothing actually sets the flags yet.
			SelectedInstances.Init(false, InstanceData.Num());
		}
#endif

		ReorderArray(BoneAttachmentBindings, IndexRemap);

		// Dirty bits are invalid after reorder — reset and force proxy recreation
		// so the render thread gets the reordered bindings.
		DirtyBoneAttachmentBindings.Init(false, BoneAttachmentBindings.Num());
		NumDirtyBoneAttachmentBindings = 0;
		if (BoneAttachmentSockets.Num() > 0)
		{
			MarkRenderStateDirty();
		}
	}
}

void UInstancedSkinnedMeshComponent::ApplyInheritedPerInstanceData(const UInstancedSkinnedMeshComponent* InArchetype)
{
	check(InArchetype);
	InstanceData = InArchetype->InstanceData;
	InstanceCustomData = InArchetype->InstanceCustomData;
	NumCustomDataFloats = InArchetype->NumCustomDataFloats;

	// Bone attachment sockets reference specific parent component instances
	// that are not valid for the inherited copy. Release and clear.
	for (const FBoneAttachmentBinding& Binding : BoneAttachmentBindings)
	{
		if (Binding.IsAttached())
		{
			ReleaseBoneAttachmentSocket(Binding.GetSocketIndex());
		}
	}
	BoneAttachmentBindings.Reset();
	BoneAttachmentBindings.SetNum(InstanceData.Num());
	DirtyBoneAttachmentBindings.Init(false, InstanceData.Num());
	NumDirtyBoneAttachmentBindings = 0;
}

bool UInstancedSkinnedMeshComponent::ShouldInheritPerInstanceData() const
{
	return ShouldInheritPerInstanceData(Cast<UInstancedSkinnedMeshComponent>(GetArchetype()));
}

bool UInstancedSkinnedMeshComponent::ShouldInheritPerInstanceData(const UInstancedSkinnedMeshComponent* InArchetype) const
{
	return (bInheritPerInstanceData || !bEditableWhenInherited) && InArchetype && InArchetype->IsInBlueprint() && !IsTemplate();
}

void UInstancedSkinnedMeshComponent::SetInstanceDataGPUOnly(bool bInInstancesGPUOnly)
{
	if (bIsInstanceDataGPUOnly != bInInstancesGPUOnly)
	{
		bIsInstanceDataGPUOnly = bInInstancesGPUOnly;

		if (bIsInstanceDataGPUOnly)
		{
			ClearInstances();
		}
	}
}

void UInstancedSkinnedMeshComponent::SetupNewInstanceData(FSkinnedMeshInstanceData& InOutNewInstanceData, int32 InInstanceIndex, const FTransform3f& InInstanceTransform, int32 InAnimationIndex)
{
	InOutNewInstanceData.Transform = InInstanceTransform;
	InOutNewInstanceData.AnimationIndex = InAnimationIndex;

	if (bPhysicsStateCreated && !IsAsyncDestroyPhysicsStateRunning())
	{
		const FTransform WorldTransform = FTransform(InInstanceTransform) * GetComponentTransform();
		InstancePhysicsBodies->Insert(InInstanceIndex, WorldTransform);
	}
}

const Nanite::FResources* UInstancedSkinnedMeshComponent::GetNaniteResources() const
{
	return Super::GetNaniteResources();
}

#if WITH_EDITOR

void UInstancedSkinnedMeshComponent::PreAssetCompilation()
{
	MarkRenderStateDirty();
}

void UInstancedSkinnedMeshComponent::PostAssetCompilation()
{
	InstanceDataManager.ClearChangeTracking();
	MarkRenderStateDirty();
}

#endif 

FInstanceDataManagerSourceDataDesc UInstancedSkinnedMeshComponent::GetComponentDesc(EShaderPlatform InShaderPlatform)
{
	return FInstancedSkinnedMeshComponentHelper::GetComponentDesc(*this, InShaderPlatform);
}

void UInstancedSkinnedMeshComponent::SendRenderInstanceData_Concurrent()
{
	Super::SendRenderInstanceData_Concurrent();

	if (TransformProvider)
	{
		TransformProvider->SubmitChanges();
	}

	// Flush bone attachment changes before the GPU-only early return —
	// GPU-only components still need incremental binding updates on the proxy.
	FlushBoneAttachmentSockets();

	// If instance data is entirely GPU driven, don't upload CPU instance data.
	if (bIsInstanceDataGPUOnly)
	{
		return;
	}

	// If the primitive isn't hidden update its instances.
	const bool bDetailModeAllowsRendering = true;//DetailMode <= GetCachedScalabilityCVars().DetailMode;
	// The proxy may not be created, this can happen when a SM is async loading for example.
	if (bDetailModeAllowsRendering && (ShouldRender() || bCastHiddenShadow || bAffectIndirectLightingWhileHidden || bRayTracingFarField))
	{
		if (SceneProxy != nullptr)
		{
			// Make sure the instance data proxy is up to date:
			if (InstanceDataManager.FlushChanges(GetComponentDesc(SceneProxy->GetScene().GetShaderPlatform())))
			{
				UpdateBounds();
				GetWorld()->Scene->UpdatePrimitiveInstances(this);
			}
		}
		else
		{
			UpdateBounds();
			GetWorld()->Scene->AddPrimitive(this);
		}
	}
}

int32 UInstancedSkinnedMeshComponent::AcquireBoneAttachmentSocket(UInstancedSkinnedMeshComponent* Parent, FName SocketName, bool bDeferResolve)
{
	// Find existing socket for this parent + socket name.
	for (TSparseArray<FBoneAttachmentSocketRef>::TIterator It(BoneAttachmentSockets); It; ++It)
	{
		FBoneAttachmentSocketRef& Socket = *It;
		if (Socket.Parent == Parent && Socket.SocketName == SocketName)
		{
			++Socket.RefCount;
			return It.GetIndex();
		}
	}

	// Create new socket entry.
	FBoneAttachmentSocketRef Socket;
	Socket.Parent = Parent;
	Socket.SocketName = SocketName;
	Socket.RefCount = 1;

	if (bDeferResolve)
	{
		// During Serialize, parent's mesh may not have completed PostLoad.
		// Leave Resolved as default (invalid BoneIndex, identity RefPose).
		Socket.bNeedsResolve = true;
	}
	else
	{
		if (!ResolveBoneAttachmentSocket(Socket))
		{
			return INDEX_NONE;
		}
	}

	return BoneAttachmentSockets.Add(MoveTemp(Socket));
}

bool UInstancedSkinnedMeshComponent::ResolveBoneAttachmentSocket(FBoneAttachmentSocketRef& Socket)
{
	Socket.bNeedsResolve = false;

	UInstancedSkinnedMeshComponent* Parent = Socket.Parent.Get();
	if (!Parent)
	{
		return false;
	}

	const USkinnedAsset* ParentAsset = Parent->GetSkinnedAsset();
	if (!ParentAsset)
	{
		UE_LOGF(LogInstancedSkinnedMesh, Warning, "ResolveBoneAttachmentSocket: Parent '%ls' has no skinned asset.", *Parent->GetPathName());
		return false;
	}

	Socket.Resolved.ParentComponentId = Parent->GetPrimitiveSceneId();

	const USkeletalMeshSocket* MeshSocket = ParentAsset->FindSocket(Socket.SocketName);
	const FName BoneName = MeshSocket ? MeshSocket->BoneName : Socket.SocketName;
	const int32 BoneIndex = ParentAsset->GetRefSkeleton().FindBoneIndex(BoneName);

	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogInstancedSkinnedMesh, Warning, "ResolveBoneAttachmentSocket: Socket/bone '%ls' not found on parent '%ls'",
			*Socket.SocketName.ToString(), *ParentAsset->GetPathName());
		return false;
	}

	Socket.Resolved.BoneIndex = BoneIndex;

	TConstArrayView<FMatrix44f> InvRefMatrices = ParentAsset->GetRefBasesInvMatrix();
	if (InvRefMatrices.IsValidIndex(BoneIndex))
	{
		TransposeTransform(Socket.Resolved.RefPoseMatrix, InvRefMatrices[BoneIndex].Inverse());
	}

	if (MeshSocket)
	{
		Socket.Resolved.Rotation = FQuat4f(MeshSocket->RelativeRotation.Quaternion());
		Socket.Resolved.Translation = FVector3f(MeshSocket->RelativeLocation);
	}

	return true;
}

void UInstancedSkinnedMeshComponent::ReleaseBoneAttachmentSocket(int32 SocketIndex)
{
	if (!BoneAttachmentSockets.IsValidIndex(SocketIndex) || !BoneAttachmentSockets.IsAllocated(SocketIndex))
	{
		return;
	}

	FBoneAttachmentSocketRef& Socket = BoneAttachmentSockets[SocketIndex];
	check(Socket.RefCount > 0);
	if (--Socket.RefCount == 0)
	{
		BoneAttachmentSockets.RemoveAt(SocketIndex);
	}
}

void UInstancedSkinnedMeshComponent::MarkBoneAttachmentBindingDirty(int32 InstanceIndex)
{
	if (!DirtyBoneAttachmentBindings[InstanceIndex])
	{
		DirtyBoneAttachmentBindings[InstanceIndex] = true;
		++NumDirtyBoneAttachmentBindings;
	}
	MarkRenderInstancesDirty();
}

void UInstancedSkinnedMeshComponent::ResetBoneAttachmentDirtyState()
{
	DirtyBoneAttachmentBindings.Init(false, DirtyBoneAttachmentBindings.Num());
	NumDirtyBoneAttachmentBindings = 0;
}

void UInstancedSkinnedMeshComponent::SetInstanceBoneAttachment(FPrimitiveInstanceId InstanceId, FPrimitiveInstanceId ParentInstanceId,
	UInstancedSkinnedMeshComponent* Parent, FName SocketName)
{
	if (!Parent)
	{
		return;
	}

	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		UE_LOGF(LogInstancedSkinnedMesh, Warning, "SetInstanceBoneAttachment: Invalid InstanceId.");
		return;
	}

	const int32 ChildIndex = InstanceDataManager.IdToIndex(InstanceId);

	if (!BoneAttachmentBindings.IsValidIndex(ChildIndex))
	{
		UE_LOGF(LogInstancedSkinnedMesh, Error, "SetInstanceBoneAttachment: Instance index %d out of range.", ChildIndex);
		return;
	}

	if (!ParentInstanceId.IsValid())
	{
		UE_LOGF(LogInstancedSkinnedMesh, Warning, "SetInstanceBoneAttachment: Invalid ParentInstanceId.");
		return;
	}

	// Release old socket ref if this instance was already attached.
	const FBoneAttachmentBinding OldBinding = BoneAttachmentBindings[ChildIndex];
	if (OldBinding.IsAttached())
	{
		ReleaseBoneAttachmentSocket(OldBinding.GetSocketIndex());
	}

	const int32 SocketIndex = AcquireBoneAttachmentSocket(Parent, SocketName);
	if (SocketIndex == INDEX_NONE)
	{
		BoneAttachmentBindings[ChildIndex] = FBoneAttachmentBinding();
		MarkBoneAttachmentBindingDirty(ChildIndex);
		return;
	}

	const int32 ParentIndex = ParentInstanceId.GetAsIndex();

	if (SocketIndex >= 255)
	{
		UE_LOGF(LogInstancedSkinnedMesh, Error, "SetInstanceBoneAttachment: Socket index %d exceeds maximum (254). Too many unique parent+socket combinations.", SocketIndex);
		ReleaseBoneAttachmentSocket(SocketIndex);
		BoneAttachmentBindings[ChildIndex] = FBoneAttachmentBinding();
		MarkBoneAttachmentBindingDirty(ChildIndex);
		return;
	}

	BoneAttachmentBindings[ChildIndex] = FBoneAttachmentBinding(SocketIndex, ParentIndex);
	MarkBoneAttachmentBindingDirty(ChildIndex);
}

void UInstancedSkinnedMeshComponent::ClearInstanceBoneAttachment(FPrimitiveInstanceId InstanceId)
{
	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		return;
	}

	const int32 ChildIndex = InstanceDataManager.IdToIndex(InstanceId);

	if (!BoneAttachmentBindings.IsValidIndex(ChildIndex))
	{
		return;
	}

	const FBoneAttachmentBinding Binding = BoneAttachmentBindings[ChildIndex];

	if (!Binding.IsAttached())
	{
		return;
	}

	ReleaseBoneAttachmentSocket(Binding.GetSocketIndex());

	BoneAttachmentBindings[ChildIndex] = FBoneAttachmentBinding();
	MarkBoneAttachmentBindingDirty(ChildIndex);
}

void UInstancedSkinnedMeshComponent::FlushBoneAttachmentSockets()
{
	if (NumDirtyBoneAttachmentBindings == 0)
	{
		return;
	}

	if (!GetWorld() || !GetWorld()->Scene || !SceneProxy)
	{
		return;
	}

	struct FBindingPatch { int32 Index; FBoneAttachmentBinding Binding; };
	TArray<FBindingPatch> BindingPatches;

	if (NumDirtyBoneAttachmentBindings > 0)
	{
		BindingPatches.Reserve(NumDirtyBoneAttachmentBindings);
		for (TConstSetBitIterator<> BitIt(DirtyBoneAttachmentBindings); BitIt; ++BitIt)
		{
			const int32 DirtyIndex = BitIt.GetIndex();
			if (BoneAttachmentBindings.IsValidIndex(DirtyIndex))
			{
				BindingPatches.Add({ DirtyIndex, BoneAttachmentBindings[DirtyIndex] });
			}
		}
	}

	ENQUEUE_RENDER_COMMAND(UpdateBoneAttachmentSockets)(
		[
			  Scene = GetWorld()->Scene
			, ComponentId = GetPrimitiveSceneId()
			, ResolvedSockets = GetBoneAttachmentSockets()
			, BindingPatches = MoveTemp(BindingPatches)
			, ExpectedBindingCount = BoneAttachmentBindings.Num()
		] (FRHICommandList&) mutable
	{
		FPrimitiveSceneInfo* SceneInfo = Scene->GetPrimitiveSceneInfo(ComponentId);
		if (!SceneInfo || !SceneInfo->Proxy)
		{
			return;
		}

		FSkinningSceneExtensionProxy* Proxy = SceneInfo->Proxy->GetSkinningSceneExtensionProxy();
		if (!Proxy || !Proxy->UseInstancing())
		{
			return;
		}

		FInstancedSkinningSceneExtensionProxy* InstancedProxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(Proxy);
		InstancedProxy->SetBoneAttachmentSockets(MoveTemp(ResolvedSockets));
		InstancedProxy->ResizeBoneAttachmentBindings(ExpectedBindingCount);

		for (const FBindingPatch& Patch : BindingPatches)
		{
			InstancedProxy->SetBoneAttachmentBinding(Patch.Index, Patch.Binding);
		}
	});

	ResetBoneAttachmentDirtyState();
}

TArray<FBoneAttachmentSocket> UInstancedSkinnedMeshComponent::GetBoneAttachmentSockets()
{
	TArray<FBoneAttachmentSocket> Result;
	if (BoneAttachmentSockets.Num() == 0)
	{
		return Result;
	}

	// Resolve any deferred sockets before copying to render thread.
	for (auto& Socket : BoneAttachmentSockets)
	{
		if (Socket.bNeedsResolve)
		{
			ResolveBoneAttachmentSocket(Socket);
		}
	}

	// Size to accommodate sparse indices (may have gaps from released sockets).
	Result.SetNum(BoneAttachmentSockets.GetMaxIndex());
	for (const auto& Socket : BoneAttachmentSockets)
	{
		const int32 Index = BoneAttachmentSockets.PointerToIndex(&Socket);
		Result[Index] = Socket.Resolved;
	}
	return Result;
}

void UInstancedSkinnedMeshComponent::SaveBoneAttachments(FArchive& Ar)
{
	int32 NumSockets = BoneAttachmentSockets.Num();
	Ar << NumSockets;

	for (TSparseArray<FBoneAttachmentSocketRef>::TConstIterator It(BoneAttachmentSockets); It; ++It)
	{
		TWeakObjectPtr<UInstancedSkinnedMeshComponent> Parent = It->Parent;
		FName SocketName = It->SocketName;
		int32 Index = It.GetIndex();
		Ar << Parent;
		Ar << SocketName;
		Ar << Index;
	}

	Ar << BoneAttachmentBindings;
}

void UInstancedSkinnedMeshComponent::LoadBoneAttachments(FArchive& Ar)
{
	int32 NumSockets = 0;
	Ar << NumSockets;

	TMap<int32, int32> IndexRemap;
	for (int32 SocketIndex = 0; SocketIndex < NumSockets; ++SocketIndex)
	{
		TWeakObjectPtr<UInstancedSkinnedMeshComponent> Parent;
		FName SocketName;
		int32 SavedIndex;
		Ar << Parent;
		Ar << SocketName;
		Ar << SavedIndex;

		int32 AcquiredIndex = INDEX_NONE;
		if (UInstancedSkinnedMeshComponent* ParentPtr = Parent.Get())
		{
			AcquiredIndex = AcquireBoneAttachmentSocket(ParentPtr, SocketName, /*bDeferResolve=*/ true);
		}

		if (SavedIndex != AcquiredIndex)
		{
			IndexRemap.Add(SavedIndex, AcquiredIndex);
		}
	}

	Ar << BoneAttachmentBindings;
	const int32 ExpectedBindingCount = bIsInstanceDataGPUOnly ? NumInstancesGPUOnly : InstanceData.Num();
	const int32 SerializedCount = BoneAttachmentBindings.Num();
	BoneAttachmentBindings.SetNum(ExpectedBindingCount);
	DirtyBoneAttachmentBindings.Init(false, ExpectedBindingCount);
	NumDirtyBoneAttachmentBindings = 0;

	// Reset ref counts — will recount from actual bindings during the remap pass below.
	for (auto& Socket : BoneAttachmentSockets)
	{
		Socket.RefCount = 0;
	}

	// Remap binding socket indices that changed during re-acquire, and recount refs.
	for (FBoneAttachmentBinding& Binding : BoneAttachmentBindings)
	{
		if (!Binding.IsAttached())
		{
			continue;
		}

		const int32 OldSocketIndex = Binding.GetSocketIndex();
		const int32* NewSocketIndex = IndexRemap.Find(OldSocketIndex);

		if (NewSocketIndex)
		{
			Binding = (*NewSocketIndex != INDEX_NONE)
				? FBoneAttachmentBinding(*NewSocketIndex, Binding.GetParentInstanceIndex())
				: FBoneAttachmentBinding();
		}
		else if (!BoneAttachmentSockets.IsAllocated(OldSocketIndex))
		{
			Binding = FBoneAttachmentBinding();
		}

		// Count actual reference (after potential remap).
		if (Binding.IsAttached())
		{
			const int32 FinalSocketIndex = Binding.GetSocketIndex();
			if (BoneAttachmentSockets.IsAllocated(FinalSocketIndex))
			{
				BoneAttachmentSockets[FinalSocketIndex].RefCount++;
			}
		}
	}

	// Remove any sockets with zero references.
	for (auto It = BoneAttachmentSockets.CreateIterator(); It; ++It)
	{
		if (It->RefCount == 0)
		{
			It.RemoveCurrent();
		}
	}
}

bool UInstancedSkinnedMeshComponent::IsHLODRelevant() const
{
	if (!CanBeHLODRelevant(this))
	{
		return false;
	}

	if (!GetSkinnedAsset())
	{
		return false;
	}

	if (!IsVisible())
	{
		return false;
	}

	if (Mobility == EComponentMobility::Movable)
	{
		return false;
	}

#if WITH_EDITORONLY_DATA
	if (!bEnableAutoLODGeneration)
	{
		return false;
	}
#endif

	return true;
}

#if WITH_EDITOR
void UInstancedSkinnedMeshComponent::ComputeHLODHash(FHLODHashBuilder& HashBuilder) const
{
	Super::ComputeHLODHash(HashBuilder);

	HashBuilder.HashField(InstanceData, GET_MEMBER_NAME_CHECKED(UInstancedSkinnedMeshComponent, InstanceData));
	HashBuilder.HashField(GetTransformProvider(), TEXT("TransformProvider"));
	HashBuilder.HashField(InstanceCustomData, GET_MEMBER_NAME_CHECKED(UInstancedSkinnedMeshComponent, InstanceCustomData));
	HashBuilder.HashField(InstanceMinDrawDistance, GET_MEMBER_NAME_CHECKED(UInstancedSkinnedMeshComponent, InstanceMinDrawDistance));
	HashBuilder.HashField(InstanceStartCullDistance, GET_MEMBER_NAME_CHECKED(UInstancedSkinnedMeshComponent, InstanceStartCullDistance));
	HashBuilder.HashField(InstanceEndCullDistance, GET_MEMBER_NAME_CHECKED(UInstancedSkinnedMeshComponent, InstanceEndCullDistance));

	HashBuilder << GetSkinnedAsset();
}
#endif

void UInstancedSkinnedMeshComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	MeshObjectFactory = &CreateInstancedSkeletalMeshObjectFunction;
	Super::CreateRenderState_Concurrent(Context);

	// Bone attachment data is populated on the proxy at CreateSceneProxy() time.
	// Clear dirty state since the proxy has a fresh full copy.
	ResetBoneAttachmentDirtyState();
}

void UInstancedSkinnedMeshComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();
}

FPrimitiveSceneProxy* UInstancedSkinnedMeshComponent::CreateSceneProxy()
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);

	ERHIFeatureLevel::Type SceneFeatureLevel = GetWorld()->GetFeatureLevel();
	FPrimitiveSceneProxy* Result = nullptr;
	FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();

#if WITH_EDITOR
	if (!bIsInstanceDataApplyCompleted)
	{
		return nullptr;
	}
#endif

	const USkinnedAsset* SkinnedAssetPtr = GetSkinnedAsset();
	if (GetInstanceCount() == 0 || SkinnedAssetPtr == nullptr || SkinnedAssetPtr->IsCompiling())
	{
		return nullptr;
	}

	if (TransformProvider != nullptr && TransformProvider->IsEnabled())
	{
		if (TransformProvider->IsCompiling())
		{
			return nullptr;
		}

		TransformProvider->SubmitChanges();
	}

	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogInstancedSkinnedMesh, Verbose, "Skipping CreateSceneProxy for UInstancedSkinnedMeshComponent %ls (UInstancedSkinnedMeshComponent PSOs are still compiling)", *GetFullName());
		return nullptr;
	}

	GetOrCreateInstanceDataSceneProxy();

	Result = FInstancedSkinnedMeshSceneProxyDesc::CreateSceneProxy(FInstancedSkinnedMeshSceneProxyDesc(this), bHideSkin, ShouldNaniteSkin(), IsEnabled());

	// Unclear exactly how this is supposed to work with a non-instanced proxy - will be interesting...
	// If GPU-only flag set, instance data is entirely GPU driven, don't upload from CPU.
	if (Result && !bIsInstanceDataGPUOnly)
	{
		InstanceDataManager.FlushChanges(GetComponentDesc(Result->GetScene().GetShaderPlatform()));
	}

	// Populate bone attachment data directly on the proxy (already resolved at acquire time).
	if (Result && BoneAttachmentSockets.Num() > 0)
	{
		FSkinningSceneExtensionProxy* SkinProxy = Result->GetSkinningSceneExtensionProxy();

		if (SkinProxy && SkinProxy->UseInstancing())
		{
			FInstancedSkinningSceneExtensionProxy* InstancedProxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(SkinProxy);
			InstancedProxy->SetBoneAttachmentSockets(GetBoneAttachmentSockets());
			InstancedProxy->SetBoneAttachmentBindings(TArray<FBoneAttachmentBinding>(BoneAttachmentBindings));
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	SendRenderDebugPhysics(Result);
#endif

	return Result;
}

void UInstancedSkinnedMeshComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	Super::OnUpdateTransform(UpdateTransformFlags | EUpdateTransformFlags::SkipPhysicsUpdate, Teleport);
	InstanceDataManager.PrimitiveTransformChanged();

	const bool bTeleport = TeleportEnumToFlag(Teleport);
	const bool bUpdateBodies = bPhysicsStateCreated && !(EUpdateTransformFlags::SkipPhysicsUpdate & UpdateTransformFlags);

	if (bUpdateBodies)
	{
		for (int32 InstanceIndex = 0; InstanceIndex < InstanceData.Num(); InstanceIndex++)
		{
			const FTransform NewInstanceTransform(FTransform(InstanceData[InstanceIndex].Transform) * GetComponentTransform());
			UpdateInstanceBodyTransform(InstanceIndex, NewInstanceTransform, bTeleport);
		}
	}
}

#if WITH_EDITOR

void UInstancedSkinnedMeshComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// Always clear the change tracking because in the editor, attributes may have been set without any sort of notification
	InstanceDataManager.ClearChangeTracking();
	if (PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedSkinnedMeshComponent, InstanceData))
		{
			if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd
				|| PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
			{
				int32 AddedAtIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
				check(AddedAtIndex != INDEX_NONE);

				AddInstanceInternal(
					AddedAtIndex,
					PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd ? FTransform::Identity : FTransform(InstanceData[AddedAtIndex].Transform),
					PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd ? 0 : InstanceData[AddedAtIndex].AnimationIndex,
					/*bWorldSpace*/false
				);
			}
			else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove)
			{
				int32 RemovedAtIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
				check(RemovedAtIndex != INDEX_NONE);

				RemoveInstanceInternal(RemovedAtIndex, true);
			}
			else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
			{
				ClearInstances();
			}
			else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
			{
			}
			MarkRenderStateDirty();
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FSkinnedMeshInstanceData, Transform)
			 || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FSkinnedMeshInstanceData, AnimationIndex))
		{
			MarkRenderStateDirty();
		}
		else if (PropertyChangedEvent.Property->GetFName() == "NumCustomDataFloats")
		{
			SetNumCustomDataFloats(NumCustomDataFloats);
		}
		else if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName() == "InstanceCustomData")
		{
			int32 ChangedCustomValueIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
			if (ensure(NumCustomDataFloats > 0))
			{
				int32 InstanceIndex = ChangedCustomValueIndex / NumCustomDataFloats;
			}
			MarkRenderStateDirty();
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedSkinnedMeshComponent, TransformProvider))
		{
			MarkRenderStateDirty();
		}
	}
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

void UInstancedSkinnedMeshComponent::PostEditUndo()
{
	Super::PostEditUndo();

	// Bone attachment arrays aren't UPROPERTYs — they aren't restored by the transaction system.
	// Clear everything and let gameplay code re-establish attachments if needed.
	for (const FBoneAttachmentBinding& Binding : BoneAttachmentBindings)
	{
		if (Binding.IsAttached())
		{
			ReleaseBoneAttachmentSocket(Binding.GetSocketIndex());
		}
	}
	BoneAttachmentBindings.Reset();
	BoneAttachmentBindings.SetNum(InstanceData.Num());
	DirtyBoneAttachmentBindings.Init(false, InstanceData.Num());
	NumDirtyBoneAttachmentBindings = 0;

	MarkRenderStateDirty();
}

void UInstancedSkinnedMeshComponent::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	if (TransformProvider != nullptr && TransformProvider->IsEnabled())
	{
		TransformProvider->BeginCacheForCookedPlatformData(TargetPlatform);
	}
}

bool UInstancedSkinnedMeshComponent::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	if (TransformProvider != nullptr)
	{
		if (!TransformProvider->IsCachedCookedPlatformDataLoaded(TargetPlatform))
		{
			return false;
		}
	}

	return Super::IsCachedCookedPlatformDataLoaded(TargetPlatform);
}

#endif

TStructOnScope<FActorComponentInstanceData> UInstancedSkinnedMeshComponent::GetComponentInstanceData() const
{
	TStructOnScope<FActorComponentInstanceData> ComponentInstanceData;
#if WITH_EDITOR
	ComponentInstanceData.InitializeAs<FInstancedSkinnedMeshComponentInstanceData>(this);
	FInstancedSkinnedMeshComponentInstanceData* SkinnedMeshInstanceData = ComponentInstanceData.Cast<FInstancedSkinnedMeshComponentInstanceData>();

	// Back up per-instance info (this is strictly for Comparison in UInstancedSkinnedMeshComponent::ApplyComponentInstanceData 
	// as this Property will get serialized by base class FActorComponentInstanceData through FComponentPropertyWriter which uses the PPF_ForceTaggedSerialization to backup all properties even the custom serialized ones
	SkinnedMeshInstanceData->InstanceData = InstanceData;

	// Back up instance selection
	SkinnedMeshInstanceData->SelectedInstances = SelectedInstances;

	// Back up per-instance hit proxies
	SkinnedMeshInstanceData->bHasPerInstanceHitProxies = bHasPerInstanceHitProxies;
#endif
	return ComponentInstanceData;
}

void UInstancedSkinnedMeshComponent::SetCullDistances(int32 StartCullDistance, int32 EndCullDistance)
{
	if (InstanceStartCullDistance != StartCullDistance || InstanceEndCullDistance != EndCullDistance)
	{
		InstanceStartCullDistance = StartCullDistance;
		InstanceEndCullDistance = EndCullDistance;

		if (GetScene() && SceneProxy)
		{
			GetScene()->UpdateInstanceCullDistance(this, StartCullDistance, EndCullDistance);
		}
	}
}

void UInstancedSkinnedMeshComponent::SetAnimationMinScreenSize(float InAnimationMinScreenSize)
{
	if (AnimationMinScreenSize != InAnimationMinScreenSize)
	{
		AnimationMinScreenSize = InAnimationMinScreenSize;

		MarkRenderStateDirty();
	}
}

void UInstancedSkinnedMeshComponent::PreApplyComponentInstanceData(struct FInstancedSkinnedMeshComponentInstanceData* InstancedMeshData)
{
#if WITH_EDITOR
	// Prevent proxy recreate while traversing the ::ApplyToComponent stack
	bIsInstanceDataApplyCompleted = false;
#endif
}

void UInstancedSkinnedMeshComponent::ApplyComponentInstanceData(struct FInstancedSkinnedMeshComponentInstanceData* InstancedMeshData)
{
#if WITH_EDITOR
	check(InstancedMeshData);

	ON_SCOPE_EXIT
	{
		bIsInstanceDataApplyCompleted = true;
	};

	if (GetSkinnedAsset() != InstancedMeshData->SkinnedAsset)
	{
		return;
	}

	// If we should inherit from archetype do it here after data was applied and before comparing (RerunConstructionScript will serialize SkipSerialization properties and reapply them even if we want to inherit them)
	const UInstancedSkinnedMeshComponent* Archetype = Cast<UInstancedSkinnedMeshComponent>(GetArchetype());
	if (ShouldInheritPerInstanceData(Archetype))
	{
		ApplyInheritedPerInstanceData(Archetype);
	}

	SelectedInstances = InstancedMeshData->SelectedInstances;
	bHasPerInstanceHitProxies = InstancedMeshData->bHasPerInstanceHitProxies;
	PrimitiveBoundsOverride = InstancedMeshData->PrimitiveBoundsOverride;
	bIsInstanceDataGPUOnly = InstancedMeshData->bIsInstanceDataGPUOnly;
	NumInstancesGPUOnly = InstancedMeshData->NumInstancesGPUOnly;

	// Sync the instance data manager after instance data may have changed via ApplyInheritedPerInstanceData.
	// Without this, the manager's instance count can be stale, causing incorrect tracking of subsequent
	// Add/Remove operations. Mirrors what UInstancedStaticMeshComponent does with Invalidate().
	// Skip for GPU-only mode: InstanceData is empty and the manager is driven by GPU-side data.
	if (!bIsInstanceDataGPUOnly)
	{
		InstanceDataManager.Reset(InstanceData.Num());
		InstanceDataManager.ClearChangeTracking();
	}
	MarkRenderStateDirty();
#endif
}

FBoxSphereBounds UInstancedSkinnedMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (PrimitiveBoundsOverride.IsValid)
	{
		return PrimitiveBoundsOverride.InverseTransformBy(GetComponentTransform().Inverse() * LocalToWorld);
	}
	else
	{
		return FInstancedSkinnedMeshComponentHelper::CalcBounds(*this, LocalToWorld);
	}
}

void UInstancedSkinnedMeshComponent::SetSkinnedAssetCallback()
{
	MergedBodySetup = nullptr;
	MarkRenderStateDirty();
}

void UInstancedSkinnedMeshComponent::RefreshBoneTransforms(FActorComponentTickFunction* TickFunction)
{
	// Can't do anything without a SkinnedAsset
	if (!GetSkinnedAsset())
	{
		return;
	}

	// Do nothing more if no bones in skeleton.
	if (GetNumComponentSpaceTransforms() == 0)
	{
		return;
	}

	UpdateBounds();
	MarkRenderTransformDirty();
	MarkRenderDynamicDataDirty();
}

void UInstancedSkinnedMeshComponent::SetNumGPUInstances(int32 InCount)
{
	if (!bIsInstanceDataGPUOnly)
	{
		UE_LOGF(LogInstancedSkinnedMesh, Error, "SetNumGPUInstances: Component '%ls' is not in GPU-only mode. Call SetInstanceDataGPUOnly(true) first.", *GetPathName());
		return;
	}

	NumInstancesGPUOnly = InCount;

	// Release socket refs for any bindings being truncated.
	const int32 OldNum = BoneAttachmentBindings.Num();
	for (int32 i = InCount; i < OldNum; ++i)
	{
		const FBoneAttachmentBinding& Binding = BoneAttachmentBindings[i];
		if (Binding.IsAttached())
		{
			ReleaseBoneAttachmentSocket(Binding.GetSocketIndex());
		}
	}

	// Resize bone attachment parallel arrays to match.
	BoneAttachmentBindings.SetNum(InCount);
	DirtyBoneAttachmentBindings.Init(true, InCount);
	NumDirtyBoneAttachmentBindings = InCount;

	// Trigger flush of resized bindings to the render proxy.
	if (BoneAttachmentSockets.Num() > 0)
	{
		MarkRenderInstancesDirty();
	}
}

FPrimitiveInstanceId UInstancedSkinnedMeshComponent::AddInstance(const FTransform& InstanceTransform, int32 AnimationIndex, bool bWorldSpace)
{
	return AddInstanceInternal(InstanceData.Num(), InstanceTransform, AnimationIndex, bWorldSpace);
}

TArray<FPrimitiveInstanceId> UInstancedSkinnedMeshComponent::AddInstances(const TArray<FTransform>& Transforms, const TArray<int32>& AnimationIndices, bool bShouldReturnIds, bool bWorldSpace)
{
	TArray<FPrimitiveInstanceId> NewInstanceIds;
	if (Transforms.IsEmpty() || (Transforms.Num() != AnimationIndices.Num()))
	{
		return NewInstanceIds;
	}

	Modify();

	const int32 NumToAdd = Transforms.Num();

	if (bShouldReturnIds)
	{
		NewInstanceIds.SetNumUninitialized(NumToAdd);
	}

	// Reserve memory space
	const int32 NewNumInstances = InstanceData.Num() + NumToAdd;
	InstanceData.Reserve(NewNumInstances);
	InstanceCustomData.Reserve(NumCustomDataFloats * NewNumInstances);
#if WITH_EDITOR
	SelectedInstances.Reserve(NewNumInstances);
#endif

	for (int32 AddIndex = 0; AddIndex < NumToAdd; ++AddIndex)
	{
		const FTransform& Transform = Transforms[AddIndex];
		const int32 AnimationIndex = AnimationIndices[AddIndex];
		FPrimitiveInstanceId InstanceId = AddInstanceInternal(InstanceData.Num(), Transform, AnimationIndex, bWorldSpace);
		if (bShouldReturnIds)
		{
			NewInstanceIds[AddIndex] = InstanceId;
		}
	}

	return NewInstanceIds;
}

bool UInstancedSkinnedMeshComponent::SetCustomDataValue(FPrimitiveInstanceId InstanceId, int32 CustomDataIndex, float CustomDataValue)
{
	if (!InstanceDataManager.IsValidId(InstanceId) || CustomDataIndex < 0 || CustomDataIndex >= NumCustomDataFloats)
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	Modify();

	InstanceDataManager.CustomDataChanged(InstanceIndex);
	InstanceCustomData[InstanceIndex * NumCustomDataFloats + CustomDataIndex] = CustomDataValue;

	return true;
}

bool UInstancedSkinnedMeshComponent::SetCustomData(FPrimitiveInstanceId InstanceId, TArrayView<const float> CustomDataFloats)
{
	if (!InstanceDataManager.IsValidId(InstanceId) || CustomDataFloats.Num() == 0)
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	Modify();

	const int32 NumToCopy = FMath::Min(CustomDataFloats.Num(), NumCustomDataFloats);
	InstanceDataManager.CustomDataChanged(InstanceIndex);
	FMemory::Memcpy(&InstanceCustomData[InstanceIndex * NumCustomDataFloats], CustomDataFloats.GetData(), NumToCopy * CustomDataFloats.GetTypeSize());
	return true;
}

void UInstancedSkinnedMeshComponent::SetNumCustomDataFloats(int32 InNumCustomDataFloats)
{
	if (FMath::Max(InNumCustomDataFloats, 0) != NumCustomDataFloats)
	{
		NumCustomDataFloats = FMath::Max(InNumCustomDataFloats, 0);
	}

	if (InstanceData.Num() * NumCustomDataFloats != InstanceCustomData.Num())
	{
		InstanceDataManager.NumCustomDataChanged();

		// Clear out and reinit to 0
		InstanceCustomData.Empty(InstanceData.Num() * NumCustomDataFloats);
		InstanceCustomData.SetNumZeroed(InstanceData.Num() * NumCustomDataFloats);
	}
}

bool UInstancedSkinnedMeshComponent::GetCustomData(FPrimitiveInstanceId InstanceId, TArrayView<float> CustomDataFloats) const
{
	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	const int32 NumToCopy = FMath::Min(CustomDataFloats.Num(), NumCustomDataFloats);
	FMemory::Memcpy(CustomDataFloats.GetData(), &InstanceCustomData[InstanceIndex * NumCustomDataFloats], NumToCopy * CustomDataFloats.GetTypeSize());
	return true;
}

bool UInstancedSkinnedMeshComponent::GetInstanceTransform(FPrimitiveInstanceId InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	const FSkinnedMeshInstanceData& Instance = InstanceData[InstanceIndex];

	OutInstanceTransform = FTransform(Instance.Transform);
	if (bWorldSpace)
	{
		OutInstanceTransform = OutInstanceTransform * GetComponentTransform();
	}

	return true;
}

bool UInstancedSkinnedMeshComponent::GetInstancePrevTransform(FPrimitiveInstanceId InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
	if (!InstanceDataManager.IsValidId(InstanceId) || !bHasPreviousTransforms || PerInstancePrevTransform.Num() != InstanceData.Num())
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);

	if (bWorldSpace)
	{
		OutInstanceTransform = FTransform(PerInstancePrevTransform[InstanceIndex]) * GetComponentTransform();
	}
	else
	{
		OutInstanceTransform = FTransform(PerInstancePrevTransform[InstanceIndex]);
	}

	return true;
}

bool UInstancedSkinnedMeshComponent::SetInstanceTransform(FPrimitiveInstanceId InstanceId, const FTransform& InstanceTransform, bool bWorldSpace)
{
	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		return false;
	}

	const int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	InstanceData[InstanceIndex].Transform = FTransform3f(bWorldSpace ? InstanceTransform.GetRelativeTransform(GetComponentTransform()) : InstanceTransform);
	InstanceDataManager.TransformChanged(InstanceIndex);
	return true;
}

void UInstancedSkinnedMeshComponent::SetHasPerInstancePrevTransforms(bool bInHasPreviousTransforms)
{
	if (bHasPreviousTransforms == bInHasPreviousTransforms)
	{
		return;
	}

	bHasPreviousTransforms = bInHasPreviousTransforms;

	if (bHasPreviousTransforms)
	{
		PerInstancePrevTransform.SetNum(InstanceData.Num());

		for (int32 Index = 0; Index < InstanceData.Num(); Index++)
		{
			PerInstancePrevTransform[Index] = InstanceData[Index].Transform;
		}
	}
	else
	{
		PerInstancePrevTransform.Empty();
	}

	MarkRenderInstancesDirty();
}

bool UInstancedSkinnedMeshComponent::SetInstancePrevTransform(FPrimitiveInstanceId InstanceId, const FTransform& InstanceTransform, bool bWorldSpace)
{
	if (!InstanceDataManager.IsValidId(InstanceId) || !bHasPreviousTransforms || PerInstancePrevTransform.Num() != InstanceData.Num())
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	PerInstancePrevTransform[InstanceIndex] = FTransform3f(bWorldSpace ? InstanceTransform.GetRelativeTransform(GetComponentTransform()) : InstanceTransform);
	InstanceDataManager.TransformChanged(InstanceIndex);
	return true;
}

bool UInstancedSkinnedMeshComponent::SetInstanceTransforms(FPrimitiveInstanceId InstanceId, const FTransform& InstanceTransform, const FTransform& InstancePrevTransform, bool bWorldSpace)
{
	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	InstanceData[InstanceIndex].Transform = FTransform3f(bWorldSpace ? InstanceTransform.GetRelativeTransform(GetComponentTransform()) : InstanceTransform);
	if (bHasPreviousTransforms && PerInstancePrevTransform.Num() == InstanceData.Num())
	{
		PerInstancePrevTransform[InstanceIndex] = FTransform3f(bWorldSpace ? InstancePrevTransform.GetRelativeTransform(GetComponentTransform()) : InstancePrevTransform);
	}
	InstanceDataManager.TransformChanged(InstanceIndex);
	return true;
}

bool UInstancedSkinnedMeshComponent::GetInstanceAnimationIndex(FPrimitiveInstanceId InstanceId, int32& OutAnimationIndex) const
{
	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	OutAnimationIndex = InstanceData[InstanceIndex].AnimationIndex;
	return true;
}

bool UInstancedSkinnedMeshComponent::SetInstanceAnimationIndex(FPrimitiveInstanceId InstanceId, int32 AnimationIndex)
{
	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	InstanceData[InstanceIndex].AnimationIndex = AnimationIndex;
	InstanceDataManager.SkinningDataChanged(InstanceIndex);
	return true;
}

bool UInstancedSkinnedMeshComponent::UpdateInstance(FPrimitiveInstanceId InstanceId, const FTransform& InstanceTransform, int32 AnimationIndex, bool bWorldSpace)
{
	if (!InstanceDataManager.IsValidId(InstanceId))
	{
		return false;
	}

	int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
	FSkinnedMeshInstanceData& InstanceDataItem = InstanceData[InstanceIndex];
	InstanceDataItem.AnimationIndex = AnimationIndex;
	InstanceDataItem.Transform = FTransform3f(bWorldSpace ? InstanceTransform.GetRelativeTransform(GetComponentTransform()) : InstanceTransform);
	InstanceDataManager.SkinningDataChanged(InstanceIndex);
	InstanceDataManager.TransformChanged(InstanceIndex);
	return true;
}

bool UInstancedSkinnedMeshComponent::RemoveInstance(FPrimitiveInstanceId InstanceId)
{
	if (InstanceDataManager.IsValidId(InstanceId))
	{
		int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
		Modify();
		return RemoveInstanceInternal(InstanceIndex, false);
	}
	return false;
}

void UInstancedSkinnedMeshComponent::RemoveInstances(const TArray<FPrimitiveInstanceId>& InstancesToRemove)
{
	Modify();

	for (FPrimitiveInstanceId InstanceId : InstancesToRemove)
	{
		if (!InstanceDataManager.IsValidId(InstanceId))
		{
			continue;
		}
		int32 InstanceIndex = InstanceDataManager.IdToIndex(InstanceId);
		RemoveInstanceInternal(InstanceIndex, false);
	}
}

void UInstancedSkinnedMeshComponent::ClearInstances()
{
	Modify();

	// Clear all the per-instance data
	InstanceData.Empty();
	InstanceCustomData.Empty();
	PerInstancePrevTransform.Empty();

	BoneAttachmentBindings.Empty();
	BoneAttachmentSockets.Empty();
	DirtyBoneAttachmentBindings.Empty();
	NumDirtyBoneAttachmentBindings = 0;

#if WITH_EDITOR
	SelectedInstances.Empty();
#endif
	InstanceDataManager.ClearInstances();

	if (bPhysicsStateCreated)
	{
		ClearAllInstanceBodies();
	}
}

struct HSkinnedMeshInstance : public HHitProxy
{
	TObjectPtr<UInstancedSkinnedMeshComponent> Component;
	int32 InstanceIndex;

	DECLARE_HIT_PROXY(ENGINE_API);
	HSkinnedMeshInstance(UInstancedSkinnedMeshComponent* InComponent, int32 InInstanceIndex)
	: HHitProxy(HPP_World)
	, Component(InComponent)
	, InstanceIndex(InInstanceIndex)
	{
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(Component);
	}

	virtual FTypedElementHandle GetElementHandle() const override
	{
	#if WITH_EDITOR
		if (Component)
		{
		#if 0
			if (true)//if (CVarEnableViewportSMInstanceSelection.GetValueOnAnyThread() != 0)
			{
				// Prefer per-instance selection if available
				// This may fail to return a handle if the feature is disabled, or if per-instance editing is disabled for this component
				if (FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(Component, InstanceIndex))
				{
					return ElementHandle;
				}
			}
		#endif

			// If per-instance selection isn't possible, fallback to general per-component selection (which may choose to select the owner actor instead)
			return UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component);
		}
	#endif	// WITH_EDITOR
		return FTypedElementHandle();
	}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}
};

IMPLEMENT_HIT_PROXY(HSkinnedMeshInstance, HHitProxy);

void UInstancedSkinnedMeshComponent::CreateHitProxyData(TArray<TRefCountPtr<HHitProxy>>& HitProxies)
{
	if (GIsEditor && bHasPerInstanceHitProxies)
	{
		int32 NumProxies = InstanceData.Num();
		HitProxies.Empty(NumProxies);

		for (int32 InstanceIdx = 0; InstanceIdx < NumProxies; ++InstanceIdx)
		{
			HitProxies.Add(new HSkinnedMeshInstance(this, InstanceIdx));
		}
	}
	else
	{
		HitProxies.Empty();
	}
}

FPrimitiveInstanceId UInstancedSkinnedMeshComponent::AddInstanceInternal(int32 InstanceIndex, const FTransform& InstanceTransform, int32 AnimationIndex, bool bWorldSpace)
{
	// This happens because the editor modifies the InstanceData array _before_ callbacks. If we could change the UI to not do that we could remove this ugly hack.
	if (!InstanceData.IsValidIndex(InstanceIndex))
	{
		check(InstanceIndex == InstanceData.Num());
		InstanceData.AddDefaulted();
	}

	FPrimitiveInstanceId InstanceId = InstanceDataManager.Add(InstanceIndex);

	const FTransform3f LocalTransform = FTransform3f(bWorldSpace ? InstanceTransform.GetRelativeTransform(GetComponentTransform()) : InstanceTransform);
	SetupNewInstanceData(InstanceData[InstanceIndex], InstanceIndex, LocalTransform, AnimationIndex);

	// Add custom data to instance
	InstanceCustomData.AddZeroed(NumCustomDataFloats);

	if (bHasPreviousTransforms)
	{
		PerInstancePrevTransform.Add(LocalTransform);
	}

	BoneAttachmentBindings.Add(FBoneAttachmentBinding());
	DirtyBoneAttachmentBindings.Add(false);

#if WITH_EDITOR
	SelectedInstances.Add(false);
#endif

	return InstanceId;
}

bool UInstancedSkinnedMeshComponent::RemoveInstanceInternal(int32 InstanceIndex, bool bInstanceAlreadyRemoved)
{
	if (!ensure(bInstanceAlreadyRemoved || InstanceData.IsValidIndex(InstanceIndex)))
	{
		return false;
	}
	InstanceDataManager.RemoveAtSwap(InstanceIndex);

	// remove instance
	if (!bInstanceAlreadyRemoved)
	{
		InstanceData.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);
	}

	if (InstanceCustomData.IsValidIndex(InstanceIndex * NumCustomDataFloats))
	{
		InstanceCustomData.RemoveAtSwap(InstanceIndex * NumCustomDataFloats, NumCustomDataFloats, EAllowShrinking::No);
	}

	if (bHasPreviousTransforms)
	{
		PerInstancePrevTransform.RemoveAtSwap(InstanceIndex);
	}

#if WITH_EDITOR
	// remove selection flag if array is filled in
	if (SelectedInstances.IsValidIndex(InstanceIndex))
	{
		SelectedInstances.RemoveAtSwap(InstanceIndex);
	}
#endif

	if (BoneAttachmentBindings.IsValidIndex(InstanceIndex))
	{
		const FBoneAttachmentBinding OldBinding = BoneAttachmentBindings[InstanceIndex];
		if (OldBinding.IsAttached())
		{
			ReleaseBoneAttachmentSocket(OldBinding.GetSocketIndex());
		}

		BoneAttachmentBindings.RemoveAtSwap(InstanceIndex);

		if (DirtyBoneAttachmentBindings[InstanceIndex])
		{
			--NumDirtyBoneAttachmentBindings;
		}
		DirtyBoneAttachmentBindings.RemoveAtSwap(InstanceIndex);

		// The swapped-in element at InstanceIndex now has a different binding than the proxy expects.
		// Mark it dirty so the next flush patches the proxy. The proxy's array is 1 element too large
		// but the extra entry is Unattached and harmlessly skipped by FillAttachmentHeaders.
		if (InstanceIndex < BoneAttachmentBindings.Num())
		{
			MarkBoneAttachmentBindingDirty(InstanceIndex);
		}
	}

	// update the physics state
	if (bPhysicsStateCreated && InstancePhysicsBodies->IsValidIndex(InstanceIndex))
	{
		InstancePhysicsBodies->RemoveAtSwap(InstanceIndex);
	}

	return true;
}

TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> UInstancedSkinnedMeshComponent::GetOrCreateInstanceDataSceneProxy()
{
	if (bIsInstanceDataGPUOnly)
	{
		return CreateInstanceDataProxyGPUOnly();
	}
	else
	{
		return InstanceDataManager.GetOrCreateProxy();
	}
}

TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> UInstancedSkinnedMeshComponent::GetInstanceDataSceneProxy() const
{
	if (bIsInstanceDataGPUOnly)
	{
		return CreateInstanceDataProxyGPUOnly();
	}
	else
	{
		return const_cast<UInstancedSkinnedMeshComponent*>(this)->InstanceDataManager.GetProxy();
	}
}

TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> UInstancedSkinnedMeshComponent::CreateInstanceDataProxyGPUOnly() const
{
	FInstanceSceneDataBuffers InstanceSceneDataBuffers(/*InbInstanceDataIsGPUOnly=*/true);
	{
		FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
		FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

		InstanceSceneDataBuffers.SetPrimitiveLocalToWorld(GetRenderMatrix(), AccessTag);

		ProxyData.NumInstancesGPUOnly = GetInstanceCountGPUOnly();
		ProxyData.NumCustomDataFloats = NumCustomDataFloats;
		ProxyData.InstanceLocalBounds.SetNum(1);
		ProxyData.InstanceLocalBounds[0] = ensure(GetSkinnedAsset()) ? GetSkinnedAsset()->GetBounds() : FBox();

		ProxyData.Flags.bHasPerInstanceCustomData = ProxyData.NumCustomDataFloats > 0;

		InstanceSceneDataBuffers.EndWriteAccess(AccessTag);
		InstanceSceneDataBuffers.ValidateData();
	}

	return MakeShared<FInstanceDataSceneProxy, ESPMode::ThreadSafe>(MoveTemp(InstanceSceneDataBuffers));
}

// ---- Physics / Collision ----

UBodySetup* UInstancedSkinnedMeshComponent::GetBodySetup()
{
	if (!MergedBodySetup)
	{
		BuildMergedBodySetup();
	}
	return MergedBodySetup;
}

void UInstancedSkinnedMeshComponent::BuildMergedBodySetup()
{
	MergedBodySetup = nullptr;

	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	USkinnedAsset* const Asset = GetSkinnedAsset();
	if (!Asset || !PhysicsAsset || PhysicsAsset->SkeletalBodySetups.IsEmpty())
	{
		return;
	}

	const FReferenceSkeleton& RefSkeleton = Asset->GetRefSkeleton();
	const TArray<FTransform>& BoneSpaceTransforms = RefSkeleton.GetRawRefBonePose();
	const int32 NumBones = BoneSpaceTransforms.Num();

	// Pre-compute component-space ref-pose transforms for all bones
	TArray<FTransform> CachedComponentSpaceTransforms;
	TArray<bool> CachedTransformReady;
	CachedComponentSpaceTransforms.SetNumUninitialized(NumBones);
	CachedTransformReady.SetNumZeroed(NumBones);

	MergedBodySetup = NewObject<UBodySetup>(this);

	// Copy properties from the first valid body setup (collision trace flag, phys material, etc.)
	for (USkeletalBodySetup* SkelBodySetup : PhysicsAsset->SkeletalBodySetups)
	{
		if (SkelBodySetup)
		{
			MergedBodySetup->CopyBodyPropertiesFrom(SkelBodySetup);
			break;
		}
	}

	// Clear geometry — we'll rebuild it with transformed elements from all bones
	MergedBodySetup->AggGeom.EmptyElements();

	for (USkeletalBodySetup* SkelBodySetup : PhysicsAsset->SkeletalBodySetups)
	{
		if (!SkelBodySetup)
		{
			continue;
		}

		const int32 BoneIndex = RefSkeleton.FindBoneIndex(SkelBodySetup->BoneName);
		if (BoneIndex == INDEX_NONE)
		{
			continue;
		}

		const FTransform& BoneTransform = FAnimationRuntime::GetComponentSpaceTransformWithCache(
			RefSkeleton, BoneSpaceTransforms, BoneIndex,
			CachedComponentSpaceTransforms, CachedTransformReady);

		MergedBodySetup->AddCollisionFrom(SkelBodySetup->AggGeom);

		// Transform the newly appended elements by the bone's component-space ref-pose transform
		for (int32 ShapeType = 0; ShapeType < EAggCollisionShape::Unknown; ++ShapeType)
		{
			const EAggCollisionShape::Type Type = static_cast<EAggCollisionShape::Type>(ShapeType);
			const int32 PrevCount = SkelBodySetup->AggGeom.GetElementCount(Type);
			const int32 TotalCount = MergedBodySetup->AggGeom.GetElementCount(Type);
			for (int32 ElemIndex = TotalCount - PrevCount; ElemIndex < TotalCount; ++ElemIndex)
			{
				FKShapeElem* Elem = MergedBodySetup->AggGeom.GetElement(Type, ElemIndex);
				if (Elem)
				{
					MergedBodySetup->AggGeom.VisitShapeAndContainer(*Elem, [&BoneTransform](auto& TypedElem, auto&)
					{
						if constexpr (requires { TypedElem.SetTransform(FTransform::Identity); })
						{
							TypedElem.SetTransform(TypedElem.GetTransform() * BoneTransform);
						}
					});
				}
			}
		}
	}

	MergedBodySetup->bNeverNeedsCookedCollisionData = true;
	MergedBodySetup->CreatePhysicsMeshes();
}

bool UInstancedSkinnedMeshComponent::ShouldCreatePhysicsState() const
{
	return !bDisableCollision && (IsRegistered() || (IsPreRegistering() && AllowsAsyncPhysicsStateCreation())) && !IsBeingDestroyed() && GetSkinnedAsset() && IsCollisionEnabled();
}

void UInstancedSkinnedMeshComponent::CreateAllInstanceBodies()
{
	const int32 NumInstances = InstanceData.Num();
	if (GetBodySetup() == nullptr)
	{
		UE_LOGF(LogInstancedSkinnedMesh, Warning, "Instance Skinned Mesh Component unable to create InstanceBodies!");
		return;
	}
	
	TArray<FTransform> WorldTransforms;
	WorldTransforms.SetNumUninitialized(NumInstances);
	for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
	{
		WorldTransforms[InstanceIndex] = FTransform(InstanceData[InstanceIndex].Transform) * GetComponentTransform();
	}

	InstancePhysicsBodies->CreateAll(WorldTransforms);	
}

void UInstancedSkinnedMeshComponent::ClearAllInstanceBodies()
{
	InstancePhysicsBodies->ClearAll();
}

void UInstancedSkinnedMeshComponent::UpdateInstanceBodyTransform(int32 InstanceIndex, const FTransform& WorldSpaceInstanceTransform, bool bTeleport)
{
	check(bPhysicsStateCreated);

	InstancePhysicsBodies->UpdateTransform(InstanceIndex, WorldSpaceInstanceTransform, bTeleport);
}

void UInstancedSkinnedMeshComponent::OnCreatePhysicsState()
{
	check(InstancePhysicsBodies->Num() == 0);

	FPhysScene* PhysScene = GetWorld()->GetPhysicsScene();
	if (!PhysScene)
	{
		return;
	}

	CreateAllInstanceBodies();
	USceneComponent::OnCreatePhysicsState();
}

void UInstancedSkinnedMeshComponent::OnDestroyPhysicsState()
{
	USceneComponent::OnDestroyPhysicsState();
	ClearAllInstanceBodies();
}

void UInstancedSkinnedMeshComponent::OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects)
{
	USceneComponent::OnAsyncDestroyPhysicsStateBegin_GameThread(OutRootedObjects);

	if (BodyInstance.bNotifyRigidBodyCollision)
	{
		if (FPhysScene* PhysScene = GetWorld()->GetPhysicsScene())
		{
			PhysScene->UnRegisterForCollisionEvents(this);
		}
	}

	InstancePhysicsBodies->BeginAsyncDestroy();
}

bool UInstancedSkinnedMeshComponent::OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout)
{
	USceneComponent::OnAsyncDestroyPhysicsState(Timeout);

	InstancePhysicsBodies->AsyncDestroy();
	return true;
}

FBodyInstance* UInstancedSkinnedMeshComponent::GetBodyInstance(FName BoneName, bool bGetWelded, int32 Index) const
{
	return InstancePhysicsBodies->GetBodyInstance(Index);
}

void UInstancedSkinnedMeshComponent::RecreateInstanceBody(int32 InstanceBodyIndex)
{
	if (InstanceData.IsValidIndex(InstanceBodyIndex))
	{
		const FTransform InstanceTM = FTransform(InstanceData[InstanceBodyIndex].Transform) * GetComponentTransform();
		InstancePhysicsBodies->Recreate(InstanceBodyIndex, InstanceTM);
	}
}

Chaos::FPhysicsObject* UInstancedSkinnedMeshComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	return InstancePhysicsBodies->GetPhysicsObjectById(Id);
}

TArray<Chaos::FPhysicsObject*> UInstancedSkinnedMeshComponent::GetAllPhysicsObjects() const
{
	return InstancePhysicsBodies->GetAllPhysicsObjects();
}

bool UInstancedSkinnedMeshComponent::CanEditSimulatePhysics()
{
	return false;
}

bool UInstancedSkinnedMeshComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	return InstancePhysicsBodies->LineTrace(OutHit, Start, End, Params.bTraceComplex, Params.bReturnPhysicalMaterial);
}

bool UInstancedSkinnedMeshComponent::SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FCollisionShape& CollisionShape, bool bTraceComplex)
{
	return InstancePhysicsBodies->Sweep(OutHit, Start, End, ShapeWorldRotation, CollisionShape, bTraceComplex);
}

bool UInstancedSkinnedMeshComponent::OverlapComponent(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape) const
{
	return InstancePhysicsBodies->Overlap(Pos, Rot, CollisionShape);
}

bool UInstancedSkinnedMeshComponent::ComponentOverlapComponentImpl(UPrimitiveComponent* PrimComp, const FVector Pos, const FQuat& Quat, const FCollisionQueryParams& Params)
{
	if (PrimComp->IsA<UInstancedSkinnedMeshComponent>())
	{
		UE_LOGF(LogCollision, Warning, "ComponentOverlapComponent : (%ls) Does not support InstancedSkinnedMesh vs InstancedSkinnedMesh overlap", *PrimComp->GetPathName());
		return false;
	}

	return InstancePhysicsBodies->OverlapComponent(PrimComp->GetBodyInstance(), Pos, Quat);
}

bool UInstancedSkinnedMeshComponent::ComponentOverlapMultiImpl(TArray<FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FQuat& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionObjectQueryParams& ObjectQueryParams) const
{
	OutOverlaps.Reset();

	return InstancePhysicsBodies->OverlapMulti(OutOverlaps, Pos, Rot, TestChannel, Params, ObjectQueryParams);
}

void UInstancedSkinnedMeshComponent::OnActorEnableCollisionChanged()
{
	if (IsRegistered() && GetSkinnedAsset())
	{
		RecreatePhysicsState();
	}
}
