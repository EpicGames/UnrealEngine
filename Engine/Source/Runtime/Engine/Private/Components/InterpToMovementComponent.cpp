// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/InterpToMovementComponent.h"
#include "GameFramework/DamageType.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterpToMovementComponent)

DEFINE_LOG_CATEGORY_STATIC(LogInterpToMovementComponent, Log, All);

const float UInterpToMovementComponent::MIN_TICK_TIME = 0.0002f;

UInterpToMovementComponent::UInterpToMovementComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUpdateOnlyIfRendered = false;
	bForceSubStepping = false;

	bWantsInitializeComponent = true;
	bAutoRegisterPhysicsVolumeUpdates = false;
	bComponentShouldUpdatePhysicsVolume = false;

	MaxSimulationTimeStep = 0.05f;
	MaxSimulationIterations = 8;

	bIsWaiting = false;
	TimeMultiplier = 1.0f;	
	Duration = 1.0f;
	CurrentDirection = 1.0f;
	CurrentTime = 0.0f;
	bStopped = false;
	bPointsFinalized = false;
}

void UInterpToMovementComponent::StopMovementImmediately()
{
	bStopped = true;
	FHitResult FakeHit;
	OnInterpToStop.Broadcast(FakeHit, CurrentTime);
	// Not calling StopSimulating here (that nulls UpdatedComponent). Users can call that explicitly instead if they wish.

	Super::StopMovementImmediately();
}


void UInterpToMovementComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_InterpToMovementComponent_TickComponent);
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// skip if don't want component updated when not rendered or updated component can't move
	if (!UpdatedComponent || ShouldSkipUpdate(DeltaTime))
	{
		return;
	}
	AActor* ActorOwner = UpdatedComponent->GetOwner();
	if (!ActorOwner || !CheckStillInWorld())
	{
		return;
	}

	if (UpdatedComponent->IsSimulatingPhysics())
	{
		return;
	}
	if((bStopped == true ) || ( !IsValid(ActorOwner) ) )
	{
		return;
	}
	if( ControlPoints.Num()== 0 ) 
	{
		return;
	}

	// This will update any control points coordinates that are linked to actors.
	UpdateControlPoints(false);

	float RemainingTime = DeltaTime;
	if (SpeedMultiplier > 0.f)
	{
		RemainingTime *= SpeedMultiplier;
	}
	int32 NumBounces = 0;
	int32 Iterations = 0;
	FHitResult Hit(1.f);

	FVector WaitPos = FVector::ZeroVector;
	if (bIsWaiting == true)
	{
		WaitPos = UpdatedComponent->GetComponentLocation();
	}
	while (RemainingTime >= MIN_TICK_TIME && (Iterations < MaxSimulationIterations) && IsValid(ActorOwner) && UpdatedComponent && IsActive())
	{
		Iterations++;

		const float TimeTick = ShouldUseSubStepping() ? GetSimulationTimeStep(RemainingTime, Iterations) : RemainingTime;
		RemainingTime -= TimeTick;

		// Calculate the current alpha with this tick iteration
		const float TargetTime = FMath::Clamp(CurrentTime + ((TimeTick*TimeMultiplier)*CurrentDirection), 0.0f, 1.0f);
		FVector MoveDelta = ComputeMoveDelta(TargetTime);
		
		// Update velocity
		Velocity = MoveDelta / TimeTick;

		// Update the rotation on the spline if required
		FRotator CurrentRotation = UpdatedComponent->GetComponentRotation();
		
		// Move the component
 		if ((bPauseOnImpact == false ) && (BehaviourType != EInterpToBehaviourType::OneShot))
 		{
 			// If we can bounce, we are allowed to move out of penetrations, so use SafeMoveUpdatedComponent which does that automatically.
 			SafeMoveUpdatedComponent(MoveDelta, CurrentRotation, bSweep, Hit, TeleportType);
 		}
 		else
		{
			// If we can't bounce, then we shouldn't adjust if initially penetrating, because that should be a blocking hit that causes a hit event and stop simulation.
			TGuardValue<EMoveComponentFlags> ScopedFlagRestore(MoveComponentFlags, MoveComponentFlags | MOVECOMP_NeverIgnoreBlockingOverlaps);
			MoveUpdatedComponent(MoveDelta, CurrentRotation, bSweep, &Hit, TeleportType);
		}
		//DrawDebugPoint(GetWorld(), UpdatedComponent->GetComponentLocation(), 16, FColor::White,true,5.0f);
		// If we hit a trigger that destroyed us, abort.
		if (!IsValid(ActorOwner) || !UpdatedComponent || !IsActive())
		{
			return;
		}

		// Update current time
		float AlphaRemainder = 0.0f;
		if (bIsWaiting == false)
		{
			// Compute time used out of tick time to get to the hit
			const float TimeDeltaAtHit = TimeTick * Hit.Time;
			// Compute new time lerp alpha based on how far we moved
			CurrentTime = CalculateNewTime(CurrentTime, TimeDeltaAtHit, Hit, true, bStopped, AlphaRemainder);
		}

		// See if we moved at all
		if (Hit.Time != 0.f)
		{
			// If we were 'waiting' we are not any more - broadcast we are moving again
			if (bIsWaiting == true)
			{
				OnWaitEndDelegate.Broadcast(Hit, CurrentTime);
				bIsWaiting = false;
			}
		}

		// Handle hit result after movement
		float SubTickTimeRemaining = 0.0f;
		if (!Hit.bBlockingHit)
		{
			if (bStopped == true)
			{
				Velocity = FVector::ZeroVector;
				break;
			}

			// Handle remainder of alpha after it goes off the end, for instance if ping-pong is set and it hit the end,
			// continue with the time remaining off the end but in the reverse direction. It is similar to hitting an object in this respect.
			if (AlphaRemainder != 0.0f)
			{
				NumBounces++;
				SubTickTimeRemaining = (AlphaRemainder * Duration);
			}
		}
		else
		{
			if (HandleHitWall(Hit, TimeTick, MoveDelta))
			{
				break;
			}

			NumBounces++;
			SubTickTimeRemaining = TimeTick * (1.f - Hit.Time);
		}

		// A few initial bounces should add more time and iterations to complete most of the simulation.
		if (NumBounces <= 2 && SubTickTimeRemaining >= MIN_TICK_TIME)
		{
			RemainingTime += SubTickTimeRemaining;
			Iterations--;
		}
	}

	UpdateComponentVelocity();
}

float UInterpToMovementComponent::CalculateNewTime( float TimeNow, float Delta, FHitResult& HitResult, bool InBroadcastEvent, bool& OutStopped, float& OutTimeRemainder )
{
	OutTimeRemainder = 0.0f;
	float NewTime = TimeNow;
	OutStopped = false;
	if (bIsWaiting == false)
	{
		NewTime += ((Delta*TimeMultiplier)*CurrentDirection);
		if (NewTime >= 1.0f)
		{
			OutTimeRemainder = NewTime - 1.0f;
			if (BehaviourType == EInterpToBehaviourType::OneShot)
			{
				NewTime = 1.0f;
				OutStopped = true;
				if(InBroadcastEvent == true )
				{
					OnInterpToStop.Broadcast(HitResult, NewTime);
				}
			}
			else if(BehaviourType == EInterpToBehaviourType::Loop_Reset)
			{
				NewTime = 0.0f;
				if (InBroadcastEvent == true)
				{
					OnResetDelegate.Broadcast(HitResult, NewTime);
				}
			}
			else
			{
				FHitResult DummyHit;
				NewTime = 1.0f;
				ReverseDirection(DummyHit, NewTime, InBroadcastEvent);
			}	
		}
		else if (NewTime < 0.0f)
		{
			OutTimeRemainder = -NewTime;
			if (BehaviourType == EInterpToBehaviourType::OneShot_Reverse)
			{
				NewTime = 0.0f;
				OutStopped = true;
				if (InBroadcastEvent == true)
				{
					OnInterpToStop.Broadcast(HitResult, NewTime);
				}				
			}
			else if (BehaviourType == EInterpToBehaviourType::PingPong)
			{
				FHitResult DummyHit;
				NewTime = 0.0f;
				ReverseDirection(DummyHit, NewTime, InBroadcastEvent);
			}			
		}
	}
	return NewTime;
}

FVector UInterpToMovementComponent::ComputeMoveDelta(float InTime) const
{	
	FVector MoveDelta = FVector::ZeroVector;
	FVector NewPosition = FVector::ZeroVector;
	//Find current control point
	float Time = 0.0f;
	int32 CurrentControlPoint = INDEX_NONE;
	// Always use the end point if we are at the end 
	if (InTime >= 1.0f)
	{
		CurrentControlPoint = ControlPoints.Num() - 1;
	}
	else
	{
		for (int32 iSpline = 0; iSpline < ControlPoints.Num(); iSpline++)
		{
			float NextTime = Time + ControlPoints[iSpline].Percentage;
			if (InTime < NextTime)
			{
				CurrentControlPoint = iSpline;
				break;
			}
			Time = NextTime;
		}
	}
	// If we found a valid control point get the position between it and the next
	if (CurrentControlPoint != INDEX_NONE)
	{
		FRotator CurrentRotation = UpdatedComponent->GetComponentRotation();
		float Base = InTime - ControlPoints[CurrentControlPoint].StartTime;
		float ThisAlpha = Base / ControlPoints[CurrentControlPoint].Percentage;
		FVector BeginControlPoint = ControlPoints[CurrentControlPoint].PositionControlPoint;
		BeginControlPoint = CurrentRotation.RotateVector(BeginControlPoint) + ( ControlPoints[CurrentControlPoint].bPositionIsRelative == true ? StartLocation : FVector::ZeroVector);

		int32 NextControlPoint = FMath::Clamp(CurrentControlPoint + 1, 0, ControlPoints.Num() - 1);
		FVector EndControlPoint = ControlPoints[NextControlPoint].PositionControlPoint;
		EndControlPoint = CurrentRotation.RotateVector(EndControlPoint) + ( ControlPoints[NextControlPoint].bPositionIsRelative == true ? StartLocation : FVector::ZeroVector);

		NewPosition = FMath::Lerp(BeginControlPoint, EndControlPoint, ThisAlpha);
	}

	FVector CurrentPosition = UpdatedComponent->GetComponentLocation();
	if (CurrentPosition != NewPosition)
	{
		MoveDelta = NewPosition - CurrentPosition;
	}
	return MoveDelta;
}

void UInterpToMovementComponent::StopSimulating(const FHitResult& HitResult)
{
	SetUpdatedComponent(nullptr);
	Velocity = FVector::ZeroVector;
	OnInterpToStop.Broadcast(HitResult, CurrentTime);
}

bool UInterpToMovementComponent::HandleHitWall(const FHitResult& Hit, float Time, const FVector& MoveDelta)
{
	AActor* ActorOwner = UpdatedComponent ? UpdatedComponent->GetOwner() : NULL;
	if (!CheckStillInWorld() || !IsValid(ActorOwner))
	{
		return true;
	}
	HandleImpact(Hit, Time, MoveDelta);

	if (!IsValid(ActorOwner) || !UpdatedComponent)
	{
		return true;
	}

	return false;
}

void UInterpToMovementComponent::HandleImpact(const FHitResult& Hit, float Time, const FVector& MoveDelta)
{
	if( bPauseOnImpact == false )
	{
		switch(BehaviourType )
		{
		case EInterpToBehaviourType::OneShot:
			OnInterpToStop.Broadcast(Hit, Time);
			bStopped = true;
			StopSimulating(Hit);
			return;
		case EInterpToBehaviourType::OneShot_Reverse:
			if( CurrentDirection == -1.0f)
			{
				OnInterpToStop.Broadcast(Hit, Time);
				bStopped = true;
				StopSimulating(Hit);
				return;
			}
			else
			{
				ReverseDirection(Hit, Time, true);
			}
			break;
		case EInterpToBehaviourType::Loop_Reset:
			{
				CurrentTime = 0.0f;
				OnResetDelegate.Broadcast(Hit, CurrentTime);
			}
			break;
		default:
			ReverseDirection(Hit, Time, true);
			break;
		}		
	}
	else
	{
		if( bIsWaiting == false )
		{
			OnWaitBeginDelegate.Broadcast(Hit, Time);
			bIsWaiting = true;
		}
	}
}

bool UInterpToMovementComponent::CheckStillInWorld()
{
	if (!UpdatedComponent)
	{
		return false;
	}
	const UWorld* MyWorld = GetWorld();
	if (!MyWorld)
	{
		return false;
	}
	// check the variations of KillZ
	AWorldSettings* WorldSettings = MyWorld->GetWorldSettings(true);
	if (!WorldSettings->AreWorldBoundsChecksEnabled())
	{
		return true;
	}
	AActor* ActorOwner = UpdatedComponent->GetOwner();
	if (!IsValid(ActorOwner))
	{
		return false;
	}
	if (!bCheckIfStillInWorld)
	{
		return true;
	}
	if (ActorOwner->GetActorLocation().Z < WorldSettings->KillZ)
	{
		UDamageType const* DmgType = WorldSettings->KillZDamageType ? WorldSettings->KillZDamageType->GetDefaultObject<UDamageType>() : GetDefault<UDamageType>();
		ActorOwner->FellOutOfWorld(*DmgType);
		return false;
	}
	// Check if box has poked outside the world
	else if (UpdatedComponent && UpdatedComponent->IsRegistered())
	{
		const FBox&	Box = UpdatedComponent->Bounds.GetBox();
		if (Box.Min.X < -HALF_WORLD_MAX || Box.Max.X > HALF_WORLD_MAX ||
			Box.Min.Y < -HALF_WORLD_MAX || Box.Max.Y > HALF_WORLD_MAX ||
			Box.Min.Z < -HALF_WORLD_MAX || Box.Max.Z > HALF_WORLD_MAX)
		{
			UE_LOGF(LogInterpToMovementComponent, Warning, "%ls is outside the world bounds!", *ActorOwner->GetName());
			ActorOwner->OutsideWorldBounds();
			// not safe to use physics or collision at this point
			ActorOwner->SetActorEnableCollision(false);
			FHitResult Hit(1.f);
			StopSimulating(Hit);
			return false;
		}
	}
	return true;
}

bool UInterpToMovementComponent::ShouldUseSubStepping() const
{
	return bForceSubStepping;
}

float UInterpToMovementComponent::GetSimulationTimeStep(float RemainingTime, int32 Iterations) const
{
	if (RemainingTime > MaxSimulationTimeStep)
	{
		if (Iterations < MaxSimulationIterations)
		{
			// Subdivide moves to be no longer than MaxSimulationTimeStep seconds
			RemainingTime = FMath::Min(MaxSimulationTimeStep, RemainingTime * 0.5f);
		}
		else
		{
			// If this is the last iteration, just use all the remaining time. This is usually better than cutting things short, as the simulation won't move far enough otherwise.
			// Print a throttled warning.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			static uint32 s_WarningCount = 0;
			if ((s_WarningCount++ < 100) || (GFrameCounter & 15) == 0)
			{
				UE_LOGF(LogInterpToMovementComponent, Warning, "GetSimulationTimeStep() - Max iterations %d hit while remaining time %.6f > MaxSimulationTimeStep (%.3f) for '%ls'", MaxSimulationIterations, RemainingTime, MaxSimulationTimeStep, *GetPathNameSafe(UpdatedComponent));
			}
#endif
		}
	}

	// no less than MIN_TICK_TIME (to avoid potential divide-by-zero during simulation).
	return FMath::Max(MIN_TICK_TIME, RemainingTime);
}


void UInterpToMovementComponent::BeginPlay()
{
	Super::BeginPlay();

	FinaliseControlPoints();
}

void UInterpToMovementComponent::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);
 
 	// Need to adjust the cached starting location... (StartLocation is always absolute)
 	StartLocation += InOffset;
 
 	// ...and all the stored control point positions if the control point positions are absolute.
 	for (auto& CtrlPoint : ControlPoints)
 	{
 		if (!CtrlPoint.bPositionIsRelative)
 		{
 			CtrlPoint.PositionControlPoint += InOffset;
 		}
 	}
}

void UInterpToMovementComponent::UpdateControlPoints(bool InForceUpdate)
{
	if (UpdatedComponent != nullptr)
	{
		if (InForceUpdate == true) 
		{
			FVector BasePosition = UpdatedComponent->GetComponentLocation();
			TotalDistance = 0.0f;
			FVector CurrentPos = ControlPoints[0].PositionControlPoint;
			if (ControlPoints[0].bPositionIsRelative == true)
			{
				CurrentPos += BasePosition;
			}
			// Calculate the distances from point to point
			for (int32 ControlPoint = 0; ControlPoint < ControlPoints.Num(); ControlPoint++)
			{
				if (ControlPoint + 1 < ControlPoints.Num())
				{
					FVector NextPosition = ControlPoints[ControlPoint + 1].PositionControlPoint;
					if( ControlPoints[ControlPoint+1].bPositionIsRelative == true )
					{
						NextPosition += BasePosition;
					}
					ControlPoints[ControlPoint].DistanceToNext = (NextPosition - CurrentPos).Size();

					TotalDistance += ControlPoints[ControlPoint].DistanceToNext;
					CurrentPos = NextPosition;
				}
				else
				{
					ControlPoints[ControlPoint].DistanceToNext = 0.0f;
					ControlPoints[ControlPoint].Percentage = 1.0f;
					ControlPoints[ControlPoint].StartTime = 1.0f;
				}
			}
			float Percent = 0.0f;
			// Use the distance to determine what % of time to spend going from each point
			for (int32 ControlPoint = 0; ControlPoint < ControlPoints.Num(); ControlPoint++)
			{
				ControlPoints[ControlPoint].StartTime = Percent;
				if(ControlPoints[ControlPoint].DistanceToNext != 0.0f)
				{
					ControlPoints[ControlPoint].Percentage = ControlPoints[ControlPoint].DistanceToNext / TotalDistance;
					Percent += ControlPoints[ControlPoint].Percentage;
				}
			}
		}
	}
}

void UInterpToMovementComponent::ReverseDirection(const FHitResult& Hit, float Time, bool InBroadcastEvent)
{
	// Invert the direction we are moving 
	if (InBroadcastEvent == true)
	{
		OnInterpToReverse.Broadcast(Hit, Time);
	}
	// flip dir
	CurrentDirection = -CurrentDirection;
}


void UInterpToMovementComponent::AddControlPointPosition(FVector Pos,bool bPositionIsRelative)
{
	UE_LOGF(LogInterpToMovementComponent, Verbose, "Pos:%ls",*Pos.ToString());
	ControlPoints.Add( FInterpControlPoint(Pos,bPositionIsRelative));
}

void UInterpToMovementComponent::ResetControlPoints()
{
	bStopped = true;
	ControlPoints.Empty();
	bPointsFinalized = false;
}

void UInterpToMovementComponent::FinaliseControlPoints()
{
	if (UpdatedComponent == nullptr)
	{
		return;
	}
	if(bPointsFinalized == true)
	{
		return;
	}
	StartLocation = UpdatedComponent->GetComponentLocation();
	TimeMultiplier = 1.0f / Duration;
	if (ControlPoints.Num() != 0)
	{
		UpdateControlPoints(true);
		// Update the component location to match first control point
		FVector MoveDelta = ComputeMoveDelta(0.0f);
		FRotator CurrentRotation = UpdatedComponent->GetComponentRotation();
		FHitResult Hit(1.f);
		UpdatedComponent->MoveComponent(MoveDelta, CurrentRotation, false, &Hit);
		bPointsFinalized = true;
	}
	
}

void UInterpToMovementComponent::RestartMovement(float InitialDirection)
{	
	CurrentDirection = InitialDirection;
	CurrentTime = 0.0f;
	bIsWaiting = false;
	bStopped = false;
}

#if WITH_EDITOR
void UInterpToMovementComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (ControlPoints.Num() != 0)
	{
		UpdateControlPoints(true);
	}
}


#endif // WITH_EDITOR

