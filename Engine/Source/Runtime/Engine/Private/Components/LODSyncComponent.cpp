// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/LODSyncComponent.h"

#include "Components/PrimitiveComponent.h"
#include "LODSyncInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LODSyncComponent)

DEFINE_LOG_CATEGORY_STATIC(LogLODSync, Warning, All);

/* ULODSyncComponent interface
 *****************************************************************************/

ULODSyncComponent::ULODSyncComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTickInEditor = true;

	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bCanEverTick = true;
}

void ULODSyncComponent::OnRegister()
{
	Super::OnRegister();

	InitializeSyncComponents();

	UE_LOGF(LogLODSync, Verbose, "Initialized Sync Component");

	// don't reset to zero because it may pop
	CurrentLOD = FMath::Clamp(CurrentLOD, 0, CurrentNumLODs - 1);
}

void ULODSyncComponent::OnUnregister()
{
	UninitializeSyncComponents();

	UE_LOGF(LogLODSync, Verbose, "Uninitialized Sync Component");
	Super::OnUnregister();
}

const FComponentSync* ULODSyncComponent::GetComponentSync(const FName& InName) const
{
	if (InName != NAME_None)
	{
		for (const FComponentSync& Comp : ComponentsToSync)
		{
			if (Comp.Name == InName)
			{
				return &Comp;
			}
		}
	}

	return nullptr;
}

void ULODSyncComponent::InitializeSyncComponents()
{
	DriveComponents.Reset();
	SubComponents.Reset();

	AActor* Owner = GetOwner();
	
	// We only support primitive components that implement the ILODSyncInterface.
	TInlineComponentArray<UPrimitiveComponent*, 16> PrimitiveComponents(Owner);
	TMap<FName, int32, TInlineSetAllocator<16>> ComponentNameToIndexMap;

	const int32 NumPrimitiveComponents = PrimitiveComponents.Num();
	for (int32 Index = 0; Index < NumPrimitiveComponents; ++Index)
	{
		FName ComponentName = PrimitiveComponents[Index]->GetFName();
		ComponentNameToIndexMap.Add(ComponentName, Index);
	}

	// current num LODs start with NumLODs
	// but if nothing is set, it will be -1
	CurrentNumLODs = NumLODs;
	// if NumLODs are -1, we try to find the max number of LODs of all the components
	const bool bFindTheMaxLOD = (NumLODs == -1);
	// we find all the components of the child and add this to prerequisite
	for (const FComponentSync& CompSync : ComponentsToSync)
	{
		if (const int32* ComponentIndex = ComponentNameToIndexMap.Find(CompSync.Name))
		{
			UPrimitiveComponent* PrimComponent = PrimitiveComponents[*ComponentIndex];
			if (PrimComponent)
			{
				// We don't need to check for ImplementsInterface() since it's tagged as CannotImplementInterfaceInBlueprint
				ILODSyncInterface* LODInterface = Cast<ILODSyncInterface>(PrimComponent);
				if (LODInterface && CompSync.SyncOption != ESyncOption::Disabled)
				{
					PrimComponent->PrimaryComponentTick.AddPrerequisite(this, PrimaryComponentTick);
					SubComponents.Add(PrimComponent);

					if (CompSync.SyncOption == ESyncOption::Drive)
					{
						DriveComponents.Add(PrimComponent);

						const int LODCount = LODInterface->GetNumSyncLODs();
						UE_LOGF(LogLODSync, Verbose, "Adding new component (%ls - LODCount : %d ) to sync.", *CompSync.Name.ToString(), LODCount);
						if (bFindTheMaxLOD)
						{
							// get max lod
							CurrentNumLODs = FMath::Max(CurrentNumLODs, LODCount);
							UE_LOGF(LogLODSync, Verbose, "MaxLOD now is set to (%d ).", CurrentNumLODs);
						}
					}
				}
			}
		}
	}

	// save inverse mapping map for the case we have to reverse lookup for it
	// and smaller sets can trigger the best desired LOD, and to find what, we have to convert to generic option
	// Custom Look up is (M:N) (M <= N) 
	// Inverse look up will be to (N:M) where(M<=N)
	for (auto Iter = CustomLODMapping.CreateIterator(); Iter; ++Iter)
	{
		FLODMappingData& Data = Iter.Value();

		Data.InverseMapping.Reset();

		TMap<int32, int32> InverseMappingIndices;
		int32 MaxLOD = 0;
		for (int32 Index = 0; Index < Data.Mapping.Num();++Index)
		{
			// if same indices are used, it will use later one, (lower LOD)
			// which is what we want
			InverseMappingIndices.FindOrAdd(Data.Mapping[Index]) = Index;
			MaxLOD = FMath::Max(MaxLOD, Data.Mapping[Index]);
		}

		// it's possible we may have invalid ones between
		Data.InverseMapping.AddUninitialized(MaxLOD + 1);
		int32 LastLOD = 0;
		for (int32 Index = 0; Index <= MaxLOD; ++Index)
		{
			int32* Found = InverseMappingIndices.Find(Index);
			if (Found)
			{
				Data.InverseMapping[Index] = *Found;
				LastLOD = *Found;
			}
			else
			{
				// if there is empty slot, we want to fill between
				Data.InverseMapping[Index] = LastLOD;
			}
		}
	}

	// after initialize we update LOD
	// so that any initialization can happen with the new LOD
	UpdateLOD();
}

void ULODSyncComponent::RefreshSyncComponents()
{
	UninitializeSyncComponents();
	InitializeSyncComponents();
}

void ULODSyncComponent::UninitializeSyncComponents()
{
	for (UPrimitiveComponent* Component : SubComponents)
	{
		if (Component)
		{
			Component->PrimaryComponentTick.RemovePrerequisite(this, PrimaryComponentTick);
		}
	}

	SubComponents.Reset();
	DriveComponents.Reset();
}

void ULODSyncComponent::UpdateLOD()
{
	// update latest LOD
	// this should tick first and it will set forced LOD
	// individual component will update it correctly
	int32 HighestPriLOD = 0xff;
	// we want to ensure we have a valid set of the driving components
	if (DriveComponents.Num() > 0)
	{
		// it seems we have a situation where components becomes nullptr between registration but has the array entry
		// so we ensure this is set before we set the data. 
		bool bHaveValidSetting = false;
		
		//Make sure we respect r.ForceLod before starting to apply lodsync logic
		static IConsoleVariable* ForceLODCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForceLOD"));
		if (ForceLODCvar && ForceLODCvar->GetInt() >= 0)
		{
			HighestPriLOD = ForceLODCvar->GetInt();
			bHaveValidSetting = true;
			UE_LOGF(LogLODSync, Verbose, "LOD Sync : Using Cvar r.ForceLod Value to set lodsync [%d]", HighestPriLOD);
		}
		else if (ForcedLOD >= 0 && ForcedLOD < CurrentNumLODs)
		{
			HighestPriLOD = ForcedLOD;
			bHaveValidSetting = true;
			UE_LOGF(LogLODSync, Verbose, "LOD Sync : Using ForcedLOD [%d]", ForcedLOD);
		}
		else
		{
			// Array is in priority order (last one is highest priority)
			for (int32 DriveComponentIndex = DriveComponents.Num() - 1; DriveComponentIndex >= 0; DriveComponentIndex--)
			{
				const UPrimitiveComponent* Component = DriveComponents[DriveComponentIndex];
				if (!Component)
				{
					continue;
				}

				const ILODSyncInterface* LODInterface = CastChecked<ILODSyncInterface>(Component);
				const int32 DesiredSyncLOD = LODInterface->GetDesiredSyncLOD();

				if (DesiredSyncLOD < 0 || LODInterface->GetBestAvailableLOD() < 0)
				{
					// Can't drive from a component that doesn't know what LOD it wants or what
					// LODs it can render.
					continue;
				}

				const int32 DesiredLOD = GetSyncMappingLOD(Component->GetFName(), DesiredSyncLOD);

				if (Component->WasRecentlyRendered())
				{
					HighestPriLOD = DesiredLOD;
					bHaveValidSetting = true;

					UE_LOGF(LogLODSync, Verbose, "LOD Sync Drivers : %ls (VISIBLE) - Source LOD [%d] RemappedLOD[%d]", *GetNameSafe(Component), DesiredSyncLOD, DesiredLOD);

					// This is the best possible match, so no need to keep looking
					break;
				}

				if (!bHaveValidSetting)
				{
					// This is the best match so far, but keep looking in case there's a visible component
					HighestPriLOD = DesiredLOD;
					bHaveValidSetting = true;
				}

				UE_LOGF(LogLODSync, Verbose, "LOD Sync Drivers : %ls - Source LOD [%d] RemappedLOD[%d]", *GetNameSafe(Component), DesiredSyncLOD, DesiredLOD);
			}
		}

		if (bHaveValidSetting)
		{
			// Ensure HighestPriLOD is with in the range
			HighestPriLOD = FMath::Clamp(HighestPriLOD, MinLOD, CurrentNumLODs - 1);
			UE_LOGF(LogLODSync, Verbose, "LOD Sync : Current LOD (%d)", HighestPriLOD);

			TArray<int32, TInlineAllocator<16>> SubComponentLODs;
			// The code below ensures that all elements corresponding to non-null components are
			// written before any elements are read, so it's safe to leave them uninitialized here.
			SubComponentLODs.SetNumUninitialized(SubComponents.Num());

			bool bFirstIteration = true;
			while (HighestPriLOD < CurrentNumLODs)
			{
				// HighestPriLOD can only be used if the corresponding LOD on each component is 
				// available.
				//
				// This loop checks to make sure all the necessary LODs are available, and requests
				// that the best one be streamed in.
				bool bAllLODsAvailable = true;
				for (int32 SubComponentIndex = 0; SubComponentIndex < SubComponents.Num(); SubComponentIndex++)
				{
					UPrimitiveComponent* Component = SubComponents[SubComponentIndex];
					if (Component)
					{
						ILODSyncInterface* LODInterface = CastChecked<ILODSyncInterface>(Component);
						const int32 NewLOD = GetCustomMappingLOD(Component->GetFName(), HighestPriLOD);

						// The first iteration of the enclosing loop tests the most desired LOD, so
						// this is the one that should be requested for streaming, regardless of
						// which LOD is ultimately used for rendering.
						if (bFirstIteration)
						{
							LODInterface->SetForceStreamedLOD(NewLOD);
						}

						// If this LOD isn't available, break out and try the next one.
						// 
						// Note that if GetBestAvailableLOD returns INDEX_NONE, this condition will
						// always be false. We ignore the invalid value rather than let it prevent
						// us from setting LODs on the other components.
						if (NewLOD < LODInterface->GetBestAvailableLOD())
						{
							bAllLODsAvailable = false;

							// Don't break out if this is the first iteration, because we need to
							// request streaming on all the most desired LODs during this iteration.
							if (!bFirstIteration)
							{
								break;
							}
						}

						SubComponentLODs[SubComponentIndex] = NewLOD;
					}
				}

				bFirstIteration = false;

				if (!bAllLODsAvailable)
				{
					HighestPriLOD++;
					continue;
				}

				// Found a good set of LODs, now set them.
				for (int32 SubComponentIndex = 0; SubComponentIndex < SubComponents.Num(); SubComponentIndex++)
				{
					UPrimitiveComponent* Component = SubComponents[SubComponentIndex];
					if (Component)
					{
						ILODSyncInterface* LODInterface = CastChecked<ILODSyncInterface>(Component);
						LODInterface->SetForceRenderedLOD(SubComponentLODs[SubComponentIndex]);
					}
				}

				break;
			}
		}
	}
}

void ULODSyncComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	UpdateLOD();
}

int32 ULODSyncComponent::GetCustomMappingLOD(const FName& ComponentName, int32 CurrentWorkingLOD) const
{
	const FLODMappingData* Found = CustomLODMapping.Find(ComponentName);
	if (Found && Found->Mapping.IsValidIndex(CurrentWorkingLOD))
	{
		return Found->Mapping[CurrentWorkingLOD];
	}

	return CurrentWorkingLOD;
}

int32 ULODSyncComponent::GetSyncMappingLOD(const FName& ComponentName, int32 CurrentSourceLOD) const
{
	const FLODMappingData* Found = CustomLODMapping.Find(ComponentName);
	if (Found && Found->InverseMapping.IsValidIndex(CurrentSourceLOD))
	{
		return Found->InverseMapping[CurrentSourceLOD];
	}

	return CurrentSourceLOD;
}

FString ULODSyncComponent::GetLODSyncDebugText() const
{
	FString OutString;

	for (UPrimitiveComponent* Component : SubComponents)
	{
		if (Component)
		{
			ILODSyncInterface* LODInterface = CastChecked<ILODSyncInterface>(Component);
			const int32 StreamedLOD = LODInterface->GetForceStreamedLOD();
			const int32 RenderedLOD = LODInterface->GetForceRenderedLOD();
			const int32 DesiredSyncLOD = LODInterface->GetDesiredSyncLOD();

			if (StreamedLOD >= 0 && StreamedLOD < LODInterface->GetBestAvailableLOD())
			{
				// We're waiting for a better LOD to stream in
				OutString += FString::Printf(TEXT("%s : %d (%d, %d)\n"), *(Component->GetFName().ToString()), RenderedLOD, DesiredSyncLOD, StreamedLOD);
			}
			else if (DesiredSyncLOD >= 0)
			{
				OutString += FString::Printf(TEXT("%s : %d (%d)\n"), *(Component->GetFName().ToString()), RenderedLOD, DesiredSyncLOD);
			}
			else
			{
				OutString += FString::Printf(TEXT("%s : %d\n"), *(Component->GetFName().ToString()), RenderedLOD);
			}
		}
	}

	return OutString;
}

