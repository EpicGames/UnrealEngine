// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightComponent.cpp: LightComponent implementation.
=============================================================================*/

#include "Components/LightComponent.h"

#include "ColorManagement/ColorSpace.h"
#include "Engine/Level.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticLightingBuildContext.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "UObject/ObjectSaveContext.h"
#include "SceneInterface.h"
#include "LightSceneProxy.h"
#include "UObject/RenderingObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/UObjectIterator.h"
#include "Engine/TextureLightProfile.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "Components/PointLightComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/BillboardComponent.h"
#include "ComponentRecreateRenderStateContext.h"
#include "LocalVertexFactory.h"
#include "PSOPrecacheMaterial.h"
#include "UObject/ICookInfo.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "WorldPartition/ActorInstanceGuids.h"
#include "GameFramework/WorldSettings.h"
#include "RenderUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightComponent)

#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#endif

#define LOCTEXT_NAMESPACE "LightComponent"

#if WITH_EDITOR
static const TCHAR* GLightSpriteAssetName = TEXT("/Engine/EditorResources/LightIcons/S_LightError.S_LightError");
#endif

void FStaticShadowDepthMap::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (FApp::CanEverRender() && Data && Data->ShadowMapSizeX > 0 && Data->ShadowMapSizeY > 0 && GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5)
	{
		const static FLazyName ClassName(TEXT("FStaticShadowDepthMap"));
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FStaticShadowDepthMap"), Data->ShadowMapSizeX, Data->ShadowMapSizeY, PF_R16F)
			.SetClassName(ClassName);

		TextureRHI = RHICmdList.CreateTexture(Desc);

		uint32 DestStride = 0;
		uint8* TextureData = (uint8*)RHICmdList.GetAsImmediate().LockTexture2D(TextureRHI, 0, RLM_WriteOnly, DestStride, false);
		uint32 RowSize = Data->ShadowMapSizeX * GPixelFormats[PF_R16F].BlockBytes;

		for (int32 Y = 0; Y < Data->ShadowMapSizeY; Y++)
		{
			FMemory::Memcpy(TextureData + DestStride * Y, ((uint8*)Data->DepthSamples.GetData()) + RowSize * Y, RowSize);
		}

		RHICmdList.GetAsImmediate().UnlockTexture2D(TextureRHI, 0, false);
	}
}

void ULightComponentBase::SetCastShadows(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& CastShadows != bNewValue)
	{
		CastShadows = bNewValue;
		MarkRenderStateDirty();
	}
}

FLinearColor ULightComponentBase::GetLightColor() const
{
	return FLinearColor(LightColor);
}

void ULightComponentBase::SetCastVolumetricShadow(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bCastVolumetricShadow != bNewValue)
	{
		bCastVolumetricShadow = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponentBase::SetCastDeepShadow(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bCastDeepShadow != bNewValue)
	{
		bCastDeepShadow = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponentBase::SetAffectReflection(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bAffectReflection != bNewValue)
	{
		bAffectReflection = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponentBase::SetAffectGlobalIllumination(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bAffectGlobalIllumination != bNewValue)
	{
		bAffectGlobalIllumination = bNewValue;
		MarkRenderStateDirty();
	}
}

// Deprecated
void ULightComponentBase::SetCastRaytracedShadow(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed())
	{
		if (bNewValue && CastRaytracedShadow == ECastRayTracedShadow::Disabled)
		{
			CastRaytracedShadow = ECastRayTracedShadow::UseProjectSetting;
			MarkRenderStateDirty();
		}
		else if (!bNewValue && CastRaytracedShadow > ECastRayTracedShadow::Disabled)
		{
			CastRaytracedShadow = ECastRayTracedShadow::Disabled;
			MarkRenderStateDirty();
		}
	}
}

void ULightComponentBase::SetCastRaytracedShadows(ECastRayTracedShadow::Type bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& CastRaytracedShadow != bNewValue)
	{
		CastRaytracedShadow = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponentBase::SetSamplesPerPixel(int NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& SamplesPerPixel != NewValue)
	{
		SamplesPerPixel = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponentBase::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LevelInstanceStaticLightingSupport)
	{
		// Starting from there LightGuid is a mix of OrigignalLightGuid & ActorInstanceGuid so move the value where it belongs
		OriginalLightGuid = LightGuid;
		LightGuid.Invalidate();
	}

	if (Ar.UEVer() < VER_UE4_INVERSE_SQUARED_LIGHTS_DEFAULT)
	{
		Intensity = Brightness_DEPRECATED;
	}

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.IsLoading() && (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RayTracedShadowsType))
	{
		CastRaytracedShadow = bCastRaytracedShadow_DEPRECATED == 0? ECastRayTracedShadow::Disabled : ECastRayTracedShadow::UseProjectSetting;
	}

#if WITH_EDITOR
	if (Ar.IsSaving() && Ar.IsObjectReferenceCollector() && !Ar.IsCooking())
	{
		FSoftObjectPathSerializationScope EditorOnlyScope(ESoftObjectPathCollectType::EditorOnlyCollect);
		FSoftObjectPath SpritePath(GLightSpriteAssetName);
		Ar << SpritePath;
	}
#endif
}

/**
 * Called after duplication & serialization and before PostLoad. Used to e.g. make sure GUIDs remains globally unique.
 */
void ULightComponentBase::PostDuplicate(EDuplicateMode::Type DuplicateMode) 
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode == EDuplicateMode::Normal)
	{
		// Create new guids for light.
		UpdateLightGUIDs();
	}
}


#if WITH_EDITOR
/**
 * Called after importing property values for this object (paste, duplicate or .t3d import)
 * Allow the object to perform any cleanup for properties which shouldn't be duplicated or
 * are unsupported by the script serialization
 */
void ULightComponentBase::PostEditImport()
{
	Super::PostEditImport();
	// Create new guids for light.
	UpdateLightGUIDs();
}

void ULightComponentBase::UpdateLightSpriteTexture()
{
	if (SpriteComponent != NULL)
	{
		SpriteComponent->SetSprite(GetEditorSprite());

		float SpriteScale = GetEditorSpriteScale();
		SpriteComponent->SetRelativeScale3D(FVector(SpriteScale));
	}
}

void ULightComponentBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Update sprite 
	UpdateLightSpriteTexture();
}

#endif

/**
 * Validates light GUIDs and resets as appropriate.
 */
void ULightComponentBase::ValidateLightGUIDs()
{
	// Validate light guids.
	if (!OriginalLightGuid.IsValid())
	{
		UpdateLightGUIDs();
	}

	
	LightGuid = (OriginalLightGuid.IsValid() && GetOwner() )? FGuid::Combine( OriginalLightGuid , FActorInstanceGuid::GetActorInstanceGuid(*GetOwner())) : FGuid();
}

void ULightComponentBase::UpdateLightGUIDs()
{
	if (HasStaticShadowing())
	{
#if WITH_EDITOR
		if (IsRunningCookCommandlet())
		{
			OriginalLightGuid = FGuid::NewDeterministicGuid(GetFullName());
		}
		else
#endif
		{
			OriginalLightGuid = FGuid::NewGuid();
		}
	}
	else
	{
		OriginalLightGuid = FGuid();
	}
	LightGuid = (OriginalLightGuid.IsValid() && GetOwner() )? FGuid::Combine( OriginalLightGuid , FActorInstanceGuid::GetActorInstanceGuid(*GetOwner())) : FGuid();
}

bool ULightComponentBase::HasStaticLighting() const
{
	return (Mobility == EComponentMobility::Static) && GetOwner();
}

bool ULightComponentBase::HasStaticShadowing() const
{
	return (Mobility != EComponentMobility::Movable) && GetOwner();
}

#if WITH_EDITOR
void ULightComponentBase::PostLoad()
{
	Super::PostLoad();

	if (!HasStaticShadowing())
	{
		OriginalLightGuid.Invalidate();
		LightGuid.Invalidate();
	}
}

void ULightComponentBase::OnRegister()
{
	Super::OnRegister();

	ValidateLightGUIDs();

	if (SpriteComponent)
	{
		SpriteComponent->SpriteInfo.Category = TEXT("Lighting");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Lighting", "Lighting");

		UpdateLightSpriteTexture();
	}

#if WITH_EDITOR
	if (bAffectsWorld && HasStaticShadowing())
	{
		FStaticLightingSystemInterface::OnLightComponentRegistered.Broadcast(this);
	}
#endif

	PrecachePSOs();
}

void ULightComponentBase::OnUnregister()
{
#if WITH_EDITOR
	// Unconditional unregistration event in case we miss any changes to mobility in the middle
	FStaticLightingSystemInterface::OnLightComponentUnregistered.Broadcast(this);
#endif

	Super::OnUnregister();
}

bool ULightComponentBase::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, VolumetricScatteringIntensity))
		{
			return Mobility != EComponentMobility::Static;
		}
	}

	return Super::CanEditChange(InProperty);
}
#endif

bool ULightComponentBase::ShouldCollideWhenPlacing() const
{
	return true;
}

FBoxSphereBounds ULightComponentBase::GetPlacementExtent() const
{
	FBoxSphereBounds NewBounds;
	NewBounds.Origin = FVector::ZeroVector;
	NewBounds.BoxExtent = FVector(25.0f, 25.0f, 25.0f);
	NewBounds.SphereRadius = 12.5f;
	return NewBounds;
}

void FLightRenderParameters::MakeShaderParameters(const FViewMatrices& ViewMatrices, float Exposure, FLightShaderParameters& OutShaderParameters) const
{
	OutShaderParameters.TranslatedWorldPosition = FVector3f(ViewMatrices.GetPreViewTranslation() + WorldPosition);
	OutShaderParameters.InvRadius = InvRadius;
	OutShaderParameters.Color = FVector3f(Color) * GetLightExposureScale(Exposure);
	OutShaderParameters.FalloffExponent = FalloffExponent;
	OutShaderParameters.Direction = Direction;
	OutShaderParameters.SpecularScale = FMath::Clamp(SpecularScale, 0.f, 1.f);
	OutShaderParameters.DiffuseScale = FMath::Clamp(DiffuseScale, 0.f, 1.f);
	OutShaderParameters.Tangent = Tangent;
	OutShaderParameters.SourceRadius = SourceRadius;
	OutShaderParameters.SpotAngles = SpotAngles;
	OutShaderParameters.SoftSourceRadius = SoftSourceRadius;
	OutShaderParameters.SourceLength = SourceLength;
	OutShaderParameters.RectLightBarnCosAngle = RectLightBarnCosAngle;
	OutShaderParameters.RectLightBarnLength = RectLightBarnLength;
	OutShaderParameters.RectLightAtlasUVOffset = RectLightAtlasUVOffset;
	OutShaderParameters.RectLightAtlasUVScale = RectLightAtlasUVScale;
	OutShaderParameters.RectLightAtlasMaxLevel = RectLightAtlasMaxLevel;
	OutShaderParameters.IESAtlasIndex = IESAtlasIndex;
	OutShaderParameters.LightFunctionAtlasLightIndex = LightFunctionAtlasLightIndex;
	OutShaderParameters.bAffectsTranslucentLighting = bAffectsTranslucentLighting;
}

// match logic in InverseExposureLerp(...)
float FLightRenderParameters::GetLightExposureScale(float Exposure, float InverseExposureBlend)
{
	if (Exposure <= 0.0f)
	{
		return 1.0f;
	}

	const float Adaptation = Exposure;
	const float Alpha = InverseExposureBlend;

	// When Alpha = 0.0, we want to multiply by 1.0. when Alpha = 1.0, we want to multiply by 1/Adaptation.
	// So the lerped value is:
	//     LerpLogScale = Lerp(log(1),log(1/Adaptation),T)
	// Which is simplified as:
	//     LerpLogScale = Lerp(0,-log(Adaptation),T)
	//     LerpLogScale = -T * logAdaptation;

	const float LerpLogScale = -Alpha * log(Adaptation);
	const float Scale = exp(LerpLogScale);

	return Scale;
}

float FLightRenderParameters::GetLightExposureScale(float Exposure) const
{
	return GetLightExposureScale(Exposure, InverseExposureBlend);
}

ULightComponentBase::ULightComponentBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Brightness_DEPRECATED = 3.1415926535897932f;
	Intensity = 3.1415926535897932f;
	LightColor = FColor::White;
	VolumetricScatteringIntensity = 1.0f;
	bAffectsWorld = true;
	CastShadows = true;
	CastStaticShadows = true;
	CastDynamicShadows = true;
	CastRaytracedShadow = ECastRayTracedShadow::UseProjectSetting;
	bCastRaytracedShadow_DEPRECATED = true;
	bAffectReflection = true;
	bAffectGlobalIllumination = true;
#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif
	DeepShadowLayerDistribution = 0.5f;
	LightComponentId = FLightComponentId::GetNextId();
}

ULightComponent::FOnUpdateColorAndBrightness ULightComponent::UpdateColorAndBrightnessEvent;

/**
 * Updates/ resets light GUIDs.
 */
ULightComponent::ULightComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Temperature = 6500.0f;
	bUseTemperature = false;
	PreviewShadowMapChannel = INDEX_NONE;
	IndirectLightingIntensity = 1.0f;
	ShadowResolutionScale = 1.0f;
	ShadowBias = 0.5f;
	ShadowSlopeBias = 0.5f;
	ShadowSharpen = 0.0f;
	ContactShadowLength = 0.0f;
	ContactShadowLengthInWS = false;
	ContactShadowCastingIntensity = 1.0f;
	ContactShadowNonCastingIntensity = 0.0f;
	bAllowMegaLights = true;
	MegaLightsShadowMethod = EMegaLightsShadowMethod::Default;
	bOverrideRayEndBias = false;
	RayEndBias = 0.0f;
	bUseIESBrightness = false;
	IESBrightnessScale = 1.0f;
	IESTexture = NULL;

	bAffectTranslucentLighting = true;
	bTransmission = false;
	LightFunctionScale = FVector(1024.0f, 1024.0f, 1024.0f);

	LightFunctionFadeDistance = 100000.0f;
	DisabledBrightness = 0.5f;
	SpecularScale = 1.0f;
	DiffuseScale = 1.0f;

	bEnableLightShaftBloom = false;
	BloomScale = .2f;
	BloomThreshold = 0;
	BloomMaxBrightness = 100.0f;
	BloomTint = FColor::White;

	RayStartOffsetDepthScale = .003f;

	MaxDrawDistance = 0.0f;
	MaxDistanceFadeRange = 0.0f;
	bAddedToSceneVisible = false;
	bForceCachedShadowsForMovablePrimitives = false;

	SamplesPerPixel = 1;
}

bool ULightComponent::AffectsPrimitive(const UPrimitiveComponent* Primitive) const
{
	// Check whether the light affects the primitive's bounding volume.
	return AffectsBounds(Primitive->Bounds);
}

bool ULightComponent::AffectsBounds(const FBoxSphereBounds& InBounds) const
{
	return true;
}

bool ULightComponent::IsShadowCast(UPrimitiveComponent* Primitive) const
{
	if(Primitive->HasStaticLighting())
	{
		return CastShadows && CastStaticShadows;
	}
	else
	{
		return CastShadows && CastDynamicShadows;
	}
}

bool ULightComponent::CastsStaticShadow() const
{
	return CastShadows && CastStaticShadows;
}

bool ULightComponent::CastsDynamicShadow() const
{
	return CastShadows && CastDynamicShadows;
}

bool ULightComponent::TransmitsThroughSurfaces() const
{
	return bTransmission && CastsDynamicShadow() && !HasStaticShadowing();
}

bool ULightComponent::GetContactShadowLengthInWS() const
{
	return ContactShadowLengthInWS ? true : false;
}

float ULightComponent::GetRayEndBias() const
{
	return bOverrideRayEndBias ? FMath::Max(RayEndBias, 0.0f) : -1.0f;
}

const FStaticShadowDepthMap* ULightComponent::GetStaticShadowDepthMap() const
{
	return &StaticShadowDepthMap;
}

float ULightComponent::ComputeLightBrightness() const
{
	float LightBrightness = Intensity;
	if (IESTexture)
	{
		// When using EV100 unit, do conversion back and force so that IES brigthness can be computed on linear value
		const bool bEVUnit = GetLightUnits() == ELightUnits::EV;
		if (bEVUnit) { LightBrightness = EV100ToLuminance(Intensity); }
		
		if (bUseIESBrightness)
		{
			LightBrightness = IESTexture->Brightness * IESBrightnessScale;
		}
		LightBrightness *= IESTexture->TextureMultiplier;

		if (bEVUnit) { LightBrightness = LuminanceToEV100(LightBrightness); }
	}

	return LightBrightness;
}

#if WITH_EDITOR
void ULightComponent::SetLightBrightness(float InBrightness)
{
	if (IESTexture && IESTexture->TextureMultiplier > 0)
	{
		// When using EV100 unit, do conversion back and force so that IES brigthness can be computed on linear value
		const bool bEVUnit = GetLightUnits() == ELightUnits::EV;
		if (bEVUnit) { InBrightness = EV100ToLuminance(InBrightness); }

		if (bUseIESBrightness && IESBrightnessScale > 0)
		{
			IESTexture->Brightness = InBrightness / IESBrightnessScale / IESTexture->TextureMultiplier;
		}
		else
		{
			Intensity = InBrightness / IESTexture->TextureMultiplier;
		}

		if (bEVUnit) { Intensity = LuminanceToEV100(Intensity); }
	}
	else
	{
		Intensity = InBrightness;
	}
}
#endif // WITH_EDITOR

void ULightComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	if (Ar.UEVer() >= VER_UE4_STATIC_SHADOW_DEPTH_MAPS)
	{
		if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MapBuildDataSeparatePackage)
		{
			FLightComponentMapBuildData* LegacyData = new FLightComponentMapBuildData();
			Ar << LegacyData->DepthMap;
			LegacyData->ShadowMapChannel = ShadowMapChannel_DEPRECATED;

			FLightComponentLegacyMapBuildData LegacyLightData;
			LegacyLightData.Id = OriginalLightGuid;
			LegacyLightData.Data = LegacyData;
			GLightComponentsWithLegacyBuildData.AddAnnotation(this, MoveTemp(LegacyLightData));
		}
	}

	if( MinRoughness_DEPRECATED == 1.0f )
	{
		MinRoughness_DEPRECATED = 0.0f;
		SpecularScale = 0.0f;
	}
}

/**
 * Called after this UObject has been serialized
 */
void ULightComponent::PostLoad()
{
	Super::PostLoad();

	if (LightFunctionMaterial && HasStaticLighting())
	{
		// Light functions can only be used on dynamic lights
		ClearLightFunctionMaterial();
	}

	// we want to make sure PreviewShadowMapChannel gets into PIE unchanged
	if (!GIsEditor || !GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
	{
		PreviewShadowMapChannel = INDEX_NONE;
	}

	// EV is a logarithmic unit, so negative values are fine
	if (GetLightUnits() != ELightUnits::EV)
	{
		Intensity = FMath::Max(0.0f, Intensity);
	}

	if (GetLinkerUEVersion() < VER_UE4_LIGHTCOMPONENT_USE_IES_TEXTURE_MULTIPLIER_ON_NON_IES_BRIGHTNESS)
	{
		if(IESTexture)
		{
			Intensity /= IESTexture->TextureMultiplier; // Previous version didn't apply IES texture multiplier, so cancel out
			IESBrightnessScale = FMath::Pow(IESBrightnessScale, 2.2f); // Previous version applied 2.2 gamma to brightness scale
			IESBrightnessScale /= IESTexture->TextureMultiplier; // Previous version didn't apply IES texture multiplier, so cancel out
		}
	}

	PrecachePSOs();
}

bool ULightComponent::CanTraceDistanceFieldShadows() const
{
	return CastShadows && CastDynamicShadows && Mobility != EComponentMobility::Static && DoesProjectSupportDistanceFields();
}

#if WITH_EDITOR
void ULightComponent::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (!ObjectSaveContext.IsCooking() && !IsTemplate())
	{
		ValidateLightGUIDs();
	}
}

bool ULightComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bCastShadowsFromCinematicObjectsOnly))
		{
			return Mobility == EComponentMobility::Movable;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightingChannels))
		{
			return Mobility != EComponentMobility::Static;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionMaterial)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionFadeDistance)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, DisabledBrightness))
		{
			if (Mobility == EComponentMobility::Static)
			{
				return false;
			}
		}

		if (!CastDynamicShadows &&
			  (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ContactShadowLength)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ContactShadowLengthInWS)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ContactShadowCastingIntensity)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ContactShadowNonCastingIntensity)))
		{
			return false;
		}
		
		const bool bIsRayStartOffset = PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, RayStartOffsetDepthScale);

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bUseRayTracedDistanceFieldShadows)
			|| bIsRayStartOffset)
		{
			bool bCanEdit = CanTraceDistanceFieldShadows();

			if (bIsRayStartOffset)
			{
				bCanEdit = bCanEdit && bUseRayTracedDistanceFieldShadows;
			}

			return bCanEdit;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, CastRaytracedShadow))
		{
			return IsRayTracingEnabled();
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionFadeDistance)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, DisabledBrightness))
		{
			return LightFunctionMaterial != NULL;
		}

		if (PropertyName == TEXT("LightmassSettings"))
		{
			return Mobility != EComponentMobility::Movable;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomThreshold)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomMaxBrightness)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomTint))
		{
			return bEnableLightShaftBloom;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, Temperature))
		{
			return bUseTemperature;
		}
	}

	return Super::CanEditChange(InProperty);
}

void ULightComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	const FString PropertyName = PropertyThatChanged ? PropertyThatChanged->GetName() : TEXT("");


	if (GetLightUnits() != ELightUnits::EV)
	{
		Intensity = FMath::Max(0.0f, Intensity);
	}
	SpecularScale = FMath::Clamp( SpecularScale, 0.0f, 1.0f );
	DiffuseScale = FMath::Clamp( DiffuseScale, 0.0f, 1.0f );

	if (HasStaticLighting())
	{
		// Lightmapped lights must not have light functions
		ClearLightFunctionMaterial();
	}
#if WITH_EDITOR
	else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionMaterial))
	{
		StashedLightFunctionMaterial = nullptr;
	}
	else if (StashedLightFunctionMaterial != nullptr)
	{
		// Light has been made non-static, restore previous light function
		LightFunctionMaterial = StashedLightFunctionMaterial;
	}
#endif

	// Unbuild lighting because a property changed
	// Exclude properties that don't affect built lighting
	//@todo - make this inclusive instead of exclusive?
	if( PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, CastTranslucentShadows) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bCastShadowsFromCinematicObjectsOnly) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, CastDynamicShadows) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bAffectTranslucentLighting) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bTransmission) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, SpecularScale) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, DiffuseScale) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionMaterial) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionScale) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionFadeDistance) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, DisabledBrightness) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ShadowResolutionScale) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ShadowBias) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ShadowSlopeBias) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ShadowSharpen) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ContactShadowLength) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ContactShadowCastingIntensity) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ContactShadowNonCastingIntensity) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ContactShadowLengthInWS) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bEnableLightShaftBloom) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomScale) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomThreshold) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomMaxBrightness) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomTint) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bUseRayTracedDistanceFieldShadows) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, RayStartOffsetDepthScale) &&
		PropertyName != USceneComponent::GetVisiblePropertyName().ToString() &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightingChannels) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, VolumetricScatteringIntensity) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bCastVolumetricShadow) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bCastDeepShadow) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, CastRaytracedShadow) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bAffectReflection) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bAffectGlobalIllumination) &&
		// Point light properties that shouldn't unbuild lighting
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, SourceRadius) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, SoftSourceRadius) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, SourceLength) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, InverseExposureBlend) &&
		// Directional light properties that shouldn't unbuild lighting
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowDistanceMovableLight) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowDistanceStationaryLight) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowCascades) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, FarShadowDistance) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, FarShadowCascadeCount) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CascadeDistributionExponent) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CascadeTransitionFraction) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ShadowDistanceFadeoutFraction) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bUseInsetShadowsForMovableObjects) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DistanceFieldShadowDistance) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, LightSourceAngle) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bEnableLightShaftOcclusion) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, OcclusionMaskDarkness) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, OcclusionDepthRange) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, LightShaftOverrideDirection) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bCastModulatedShadows) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ModulatedShadowColor) &&
		PropertyName != GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ShadowAmount) &&
		// Properties that should only unbuild lighting for a Static light (can be changed dynamically on a Stationary light)
		(PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, Intensity) || Mobility == EComponentMobility::Static) &&
		(PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightColor) || Mobility == EComponentMobility::Static) &&
		(PropertyName != GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, Temperature) || Mobility == EComponentMobility::Static) )
	{
		InvalidateLightingCache();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ULightComponent::UpdateLightSpriteTexture()
{
	if( SpriteComponent != NULL )
	{
		if (HasStaticShadowing() &&
			!HasStaticLighting() &&
			bAffectsWorld &&
			CastShadows &&
			CastStaticShadows &&
			PreviewShadowMapChannel == INDEX_NONE &&
			(GetWorld() && !GetWorld()->IsPreviewWorld()))
		{
			UTexture2D* SpriteTexture = NULL;
			FCookLoadScope EditorOnlyScope(ECookLoadType::EditorOnly);
			SpriteTexture = LoadObject<UTexture2D>(NULL, GLightSpriteAssetName);
			SpriteComponent->SetSprite(SpriteTexture);
			SpriteComponent->SetRelativeScale3D(FVector(0.5f));
		}
		else
		{
			Super::UpdateLightSpriteTexture();
		}
	}
}

void ULightComponent::CheckForErrors()
{
	Super::CheckForErrors();

	const bool bUsingStaticLighting = HasStaticLighting() &&
		!GetWorld()->GetWorldSettings()->bForceNoPrecomputedLighting &&
		IsStaticLightingAllowed();
	
	static const auto CVarDynamicGIMethod = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DynamicGlobalIlluminationMethod"));
	static const auto CVarLumenAllowed = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Lumen.DiffuseIndirect.Allow"));

	const bool bLumenEnabled = CVarDynamicGIMethod->GetValueOnGameThread() == 1 &&
		CVarLumenAllowed->GetValueOnGameThread() &&
		DoesPlatformSupportLumenGI(GetScene()->GetShaderPlatform()) &&
		DoesProjectSupportLumenGI(GetScene()->GetShaderPlatform());

	if (bUsingStaticLighting && bLumenEnabled)
	{
		FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(LOCTEXT( "MapCheck_Message_StaticLightWithLumen", "is using static lighting while Lumen is enabled. Static lights will fail to render." )))
				->AddToken(FMapErrorToken::Create(FMapErrors::StaticLightWithLumen_Warning));
	}
}
#endif // WITH_EDITOR

void ULightComponent::BeginDestroy()
{
	Super::BeginDestroy();

	BeginReleaseResource(&StaticShadowDepthMap);

	// Use a fence to keep track of when the rendering thread executes the release command
	DestroyFence.BeginFence();
}

bool ULightComponent::IsReadyForFinishDestroy()
{
	// Don't allow the light component to be destroyed until its rendering resources have been released
	return Super::IsReadyForFinishDestroy() && DestroyFence.IsFenceComplete();
}

void ULightComponent::OnRegister()
{
	// Update GUIDs on attachment if they are not valid.
	ValidateLightGUIDs();

	if (OriginalLightGuid.IsValid() && !HasStaticLighting() && HasStaticShadowing())	
	{		
		// Since we support LevelInstances the channel must always be reassigned on load
		if (UMapBuildDataRegistry* DataRegistry = UMapBuildDataRegistry::Get(this))
		{
			if (FLightComponentMapBuildData* LightBuildData = DataRegistry->GetLightBuildData(LightGuid))
			{
				PreviewShadowMapChannel = LightBuildData->ShadowMapChannel;
			}
		}
	}

	Super::OnRegister();
}

void ULightComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	if (bAffectsWorld)
	{
		UWorld* World = GetWorld();
		const bool bValidIntensity = Intensity > 0.f || GetLightUnits() == ELightUnits::EV;
		const bool bHidden = !ShouldComponentAddToScene() || !ShouldRender() || !bValidIntensity;
		if (!bHidden)
		{
			InitializeStaticShadowDepthMap();

			// Add the light to the scene.
			World->Scene->AddLight(this);
			bAddedToSceneVisible = true;
		}
		// Add invisible stationary lights to the scene in the editor
		// Even invisible stationary lights consume a shadowmap channel so they must be included in the stationary light overlap preview
		else if (GIsEditor 
			&& !World->IsGameWorld()
			&& CastShadows 
			&& CastStaticShadows 
			&& HasStaticShadowing()
			&& !HasStaticLighting())
		{
			InitializeStaticShadowDepthMap();

			World->Scene->AddInvisibleLight(this);
		}
	}
}

void ULightComponent::SendRenderTransform_Concurrent()
{
	// Update the scene info's transform for this light.
	GetWorld()->Scene->UpdateLightTransform(this);
	Super::SendRenderTransform_Concurrent();
}

void ULightComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();
	UWorld* MyWorld = GetWorld();
	check(MyWorld != nullptr);
	if (ensure(MyWorld->Scene != nullptr))
	{
		MyWorld->Scene->RemoveLight(this);
	}
	bAddedToSceneVisible = false;
}

void ULightComponent::PrecachePSOs()
{
#if UE_WITH_PSO_PRECACHING
	if (!FApp::CanEverRender() || !IsComponentPSOPrecachingEnabled())
	{
		return;
	}
	if (LightFunctionMaterial)
	{
		LightFunctionMaterial->ConditionalPostLoad();

		FMaterialInterfacePSOPrecacheParams Params;
		Params.Priority = EPSOPrecachePriority::High; //< No boosting right now - can be added
		Params.MaterialInterface = LightFunctionMaterial;
		Params.VertexFactoryDataList.Add(&FLocalVertexFactory::StaticType);
				
		TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
		FGraphEventArray PSOPrecacheCompileEvents;
		PrecacheMaterialPSOs( { Params }, MaterialPSOPrecacheRequestIDs, PSOPrecacheCompileEvents);

		// Request recreate of the render state when the PSO compilation is ready (if we want to delay proxy creation)
		auto Callback = [this]()
		{
			MarkRenderStateDirty();
		};
		PSOPrecacheComponentData.SetMaterialPSOPrecacheData(this, MaterialPSOPrecacheRequestIDs, PSOPrecacheCompileEvents, Callback);
	}
#endif
}

void ULightComponent::UpdateProxy(FUpdateLightProxyCallback&& Func)
{
	if (SceneProxy)
	{
		UWorld* World = GetWorld();
		if (World && World->Scene)
		{
			World->Scene->UpdateLightProxy(this, MoveTemp(Func));
		}
	}
}

#if WITH_EDITOR
bool ULightComponent::GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty)
{
	if (ElementIndex == 0)
	{
		OutOwner = this;
		OutPropertyPath = GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionMaterial);
		OutProperty = ULightComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULightComponent, LightFunctionMaterial));
		return true;
	}

	return false;
}
#endif // WITH_EDITOR

ELightUnits ULightComponent::GetLightUnits() const { return ELightUnits::Unitless; }

/** Set brightness of the light */
void ULightComponent::SetIntensity(float NewIntensity)
{
	// Can't set brightness on a static light
	if (AreDynamicDataChangesAllowed()
		&& Intensity != NewIntensity)
	{
		Intensity = NewIntensity;

		// Use lightweight color and brightness update if possible
		UpdateColorAndBrightness();
	}
}

void ULightComponent::SetIndirectLightingIntensity(float NewIntensity)
{
	// Can't set brightness on a static light
	if (AreDynamicDataChangesAllowed()
		&& IndirectLightingIntensity != NewIntensity)
	{
		IndirectLightingIntensity = NewIntensity;

		// Use lightweight color and brightness update 
		UWorld* World = GetWorld();
		if( World && World->Scene )
		{
			//@todo - remove from scene if brightness or color becomes 0
			World->Scene->UpdateLightColorAndBrightness( this );
		}

		UpdateColorAndBrightnessEvent.Broadcast(*this);
	}
}

void ULightComponent::SetVolumetricScatteringIntensity(float NewIntensity)
{
	// Can't set brightness on a static light
	if (AreDynamicDataChangesAllowed()
		&& VolumetricScatteringIntensity != NewIntensity)
	{
		VolumetricScatteringIntensity = NewIntensity;

		// Use lightweight color and brightness update 
		UWorld* World = GetWorld();
		if( World && World->Scene )
		{
			//@todo - remove from scene if brightness or color becomes 0
			World->Scene->UpdateLightColorAndBrightness( this );
		}

		UpdateColorAndBrightnessEvent.Broadcast(*this);
	}
}

/** Set color of the light */
void ULightComponent::SetLightColor(FLinearColor NewLightColor, bool bSRGB)
{
	const FColor NewColor(NewLightColor.ToFColor(bSRGB));
	SetLightFColor(NewColor);
}

void ULightComponent::SetLightFColor(FColor NewLightColor)
{
	// Can't set color on a static light
	if (AreDynamicDataChangesAllowed()
		&& LightColor != NewLightColor)
	{
		LightColor	= NewLightColor;

		// Use lightweight color and brightness update 
		UWorld* World = GetWorld();
		if( World && World->Scene )
		{
			//@todo - remove from scene if brightness or color becomes 0
			World->Scene->UpdateLightColorAndBrightness( this );
		}

		UpdateColorAndBrightnessEvent.Broadcast(*this);
	}
}

/** Set color temperature of the light */
void ULightComponent::SetTemperature(float NewTemperature)
{
	// Can't set color on a static light
	if (AreDynamicDataChangesAllowed()
		&& Temperature != NewTemperature)
	{
		Temperature = NewTemperature;

		// Use lightweight color and brightness update 
		UWorld* World = GetWorld();
		if( World && World->Scene )
		{
			//@todo - remove from scene if brightness or color becomes 0
			World->Scene->UpdateLightColorAndBrightness( this );
		}

		UpdateColorAndBrightnessEvent.Broadcast(*this);
	}
}

void ULightComponent::SetUseTemperature(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bUseTemperature != bNewValue)
	{
		bUseTemperature = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetLightFunctionMaterial(UMaterialInterface* NewLightFunctionMaterial)
{
	// Can't set light function on a static light
	if (AreDynamicDataChangesAllowed()
		&& NewLightFunctionMaterial != LightFunctionMaterial)
	{
#if WITH_EDITOR
		StashedLightFunctionMaterial = nullptr;
#endif
		LightFunctionMaterial = NewLightFunctionMaterial;

		PrecachePSOs();

		if (SceneProxy)
		{
			SceneProxy->SetLightFunctionMaterial_GameThread(this);
		}
	}
}

void ULightComponent::ClearLightFunctionMaterial()
{
#if WITH_EDITOR
	StashedLightFunctionMaterial = LightFunctionMaterial;
#endif
	LightFunctionMaterial = nullptr;
}

void ULightComponent::SetLightFunctionScale(FVector NewLightFunctionScale)
{
	if (AreDynamicDataChangesAllowed()
		&& NewLightFunctionScale != LightFunctionScale)
	{
		LightFunctionScale = NewLightFunctionScale;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetLightFunctionFadeDistance(float NewLightFunctionFadeDistance)
{
	if (AreDynamicDataChangesAllowed()
		&& NewLightFunctionFadeDistance != LightFunctionFadeDistance)
	{
		LightFunctionFadeDistance = NewLightFunctionFadeDistance;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetLightFunctionDisabledBrightness(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& NewValue != DisabledBrightness)
	{
		DisabledBrightness = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetAffectTranslucentLighting(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bAffectTranslucentLighting != bNewValue)
	{
		bAffectTranslucentLighting = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetTransmission(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bTransmission != bNewValue)
	{
		bTransmission = bNewValue;
		if (SceneProxy)
		{
			SceneProxy->SetTransmission_GameThread(this);
		}
	}
}

void ULightComponent::SetEnableLightShaftBloom(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bEnableLightShaftBloom != bNewValue)
	{
		bEnableLightShaftBloom = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetBloomScale(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& BloomScale != NewValue)
	{
		BloomScale = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetBloomThreshold(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& BloomThreshold != NewValue)
	{
		BloomThreshold = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetBloomMaxBrightness(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& BloomMaxBrightness != NewValue)
	{
		BloomMaxBrightness = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetBloomTint(FColor NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& BloomTint != NewValue)
	{
		BloomTint = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetIESTexture(UTextureLightProfile* NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& IESTexture != NewValue)
	{
		IESTexture = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetUseIESBrightness(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bUseIESBrightness != bNewValue)
	{
		bUseIESBrightness = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetIESBrightnessScale(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& IESBrightnessScale != NewValue)
	{
		IESBrightnessScale = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetShadowBias(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& ShadowBias != NewValue)
	{
		ShadowBias = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetShadowSlopeBias(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& ShadowSlopeBias != NewValue)
	{
		ShadowSlopeBias = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetSpecularScale(float NewValue)
{
	NewValue = FMath::Clamp(NewValue, 0.f, 1.f);
	if (AreDynamicDataChangesAllowed()
		&& SpecularScale != NewValue)
	{
		SpecularScale = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetDiffuseScale(float NewValue)
{
	NewValue = FMath::Clamp(NewValue, 0.f, 1.f);
	if (AreDynamicDataChangesAllowed()
		&& DiffuseScale != NewValue)
	{
		DiffuseScale = NewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetForceCachedShadowsForMovablePrimitives(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bForceCachedShadowsForMovablePrimitives != bNewValue)
	{
		bForceCachedShadowsForMovablePrimitives = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetLightingChannels(bool bChannel0, bool bChannel1, bool bChannel2)
{
	if (bChannel0 != LightingChannels.bChannel0 ||
		bChannel1 != LightingChannels.bChannel1 ||
		bChannel2 != LightingChannels.bChannel2)
	{
		LightingChannels.bChannel0 = bChannel0;
		LightingChannels.bChannel1 = bChannel1;
		LightingChannels.bChannel2 = bChannel2;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetUseRayTracedDistanceFieldShadows(bool bNewValue)
{
	// Never set to true if not supported.
	bNewValue = bNewValue && CanTraceDistanceFieldShadows();

	if (AreDynamicDataChangesAllowed() &&
		bNewValue != bUseRayTracedDistanceFieldShadows)
	{
		bUseRayTracedDistanceFieldShadows = bNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetMaxDrawDistance(float NewValue)
{
	// Negative max draw distance has no meaning. We only process positive distances in the lighting code anyway. Clamping here to avoid marking render state dirty in case of negative input value.
	const float ClampedNewValue = FMath::Max(NewValue, 0.0f);

	if (AreDynamicDataChangesAllowed() &&
		ClampedNewValue != MaxDrawDistance)
	{
		MaxDrawDistance = ClampedNewValue;
		MarkRenderStateDirty();
	}
}

void ULightComponent::SetMaxDistanceFadeRange(float NewValue)
{
	if (AreDynamicDataChangesAllowed() &&
		NewValue != MaxDistanceFadeRange)
	{
		MaxDistanceFadeRange = NewValue;
		MarkRenderStateDirty();
	}
}

// GetDirection
FVector ULightComponent::GetDirection() const 
{ 
	return GetComponentTransform().GetUnitAxis( EAxis::X );
}

void ULightComponent::UpdateColorAndBrightness()
{
	UWorld* World = GetWorld();
	if( World && World->Scene )
	{
		const bool bValidIntensity = Intensity > 0.f || GetLightUnits() == ELightUnits::EV;
		const bool bNeedsToBeAddedToScene = (!bAddedToSceneVisible && bValidIntensity);
		const bool bNeedsToBeRemovedFromScene = (bAddedToSceneVisible && !bValidIntensity);
		if (bNeedsToBeAddedToScene || bNeedsToBeRemovedFromScene)
		{
			// We may have just been set to 0 intensity or we were previously 0 intensity.
			// Mark the render state dirty to add or remove this light from the scene as necessary.
			MarkRenderStateDirty();
		}
		else if (bAddedToSceneVisible && bValidIntensity)
		{
			// We are already in the scene. Just update with this fast path command
			World->Scene->UpdateLightColorAndBrightness(this);
		}
	}

	UpdateColorAndBrightnessEvent.Broadcast(*this);
}

//
//	ULightComponent::InvalidateLightingCache
//

void ULightComponent::InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly)
{
	if (HasStaticShadowing()) // == non movable
	{
#if WITH_EDITOR
		FStaticLightingSystemInterface::OnLightComponentUnregistered.Broadcast(this);
#endif

		// Save the light state for transactions.
		Modify();

		BeginReleaseResource(&StaticShadowDepthMap);

		// Create new guids for light.
		UpdateLightGUIDs();

#if WITH_EDITOR
		if (GIsEditor)
		{
			UWorld* World = GetWorld();
			bool bStationary = HasStaticShadowing() && !HasStaticLighting();
			if (World != NULL && bStationary)
			{
				ReassignStationaryLightChannels(World, false, NULL);
			}
		}
#endif

		MarkRenderStateDirty();

#if WITH_EDITOR
		if (bAffectsWorld)
		{
			FStaticLightingSystemInterface::OnLightComponentRegistered.Broadcast(this);
		}
#endif
	}
	else
	{
		// Movable lights will have a GUID of 0
		OriginalLightGuid.Invalidate();
		LightGuid.Invalidate();
	}
}

TStructOnScope<FActorComponentInstanceData> ULightComponent::GetComponentInstanceData() const
{
	// Allocate new struct for holding light map data
	return MakeStructOnScope<FActorComponentInstanceData, FPrecomputedLightInstanceData>(this);
}

void ULightComponent::ApplyComponentInstanceData(FPrecomputedLightInstanceData* LightMapData)
{
	check(LightMapData);

	if (!LightMapData->Transform.Equals(GetComponentTransform()))
	{
		return;
	}

	OriginalLightGuid = (HasStaticShadowing() ? LightMapData->OriginalLightGuid : FGuid());
	LightGuid = (HasStaticShadowing() ? LightMapData->LightGuid : FGuid());	
	PreviewShadowMapChannel = LightMapData->PreviewShadowMapChannel;

	MarkRenderStateDirty();

#if WITH_EDITOR
	// Update the icon with the new state of PreviewShadowMapChannel
	UpdateLightSpriteTexture();
#endif
}

void ULightComponent::PropagateLightingScenarioChange()
{
	FComponentRecreateRenderStateContext Context(this);
	BeginReleaseResource(&StaticShadowDepthMap);
}

bool ULightComponent::IsPrecomputedLightingValid() const
{
	return GetLightComponentMapBuildData() != NULL && HasStaticShadowing();
}

int32 ULightComponent::GetNumMaterials() const
{
	return 1;
}

const FLightComponentMapBuildData* ULightComponent::GetLightComponentMapBuildData() const
{
	AActor* Owner = GetOwner();

	if (Owner)
	{
		ULevel* OwnerLevel = Owner->GetLevel();

		if (OwnerLevel && OwnerLevel->OwningWorld)
		{
#if WITH_EDITOR
			if (FStaticLightingSystemInterface::GetLightComponentMapBuildData(this))
			{
				return FStaticLightingSystemInterface::GetLightComponentMapBuildData(this);
			}
#endif

			UMapBuildDataRegistry* MapBuildData = UMapBuildDataRegistry::Get(this);

			if (MapBuildData)
			{
				return MapBuildData->GetLightBuildData(LightGuid);
			}
		}
	}

	return NULL;
}

void ULightComponent::InitializeStaticShadowDepthMap()
{
	if (HasStaticShadowing() && !HasStaticLighting())
	{
		const FStaticShadowDepthMapData* DepthMapData = NULL;
		const FLightComponentMapBuildData* MapBuildData = GetLightComponentMapBuildData();
	
		if (MapBuildData)
		{
			DepthMapData = &MapBuildData->DepthMap;
		}

		FStaticShadowDepthMap* DepthMap = &StaticShadowDepthMap;
		ENQUEUE_RENDER_COMMAND(SetDepthMapData)(
			[DepthMap, DepthMapData](FRHICommandList& RHICmdList)
			{
				DepthMap->Data = DepthMapData;
			});

		BeginInitResource(&StaticShadowDepthMap);
			}
}

FLinearColor ULightComponent::GetColoredLightBrightness() const
{
	// Brightness in Lumens
	float LightBrightness = ComputeLightBrightness();
	FLinearColor Energy = FLinearColor(LightColor) * LightBrightness;
	if (bUseTemperature)
	{
		Energy *= GetColorTemperature();
	}

	return Energy;
}

FLinearColor ULightComponent::GetColorTemperature() const
{
	return UE::Color::FColorSpace::GetWorking().MakeFromColorTemperature(Temperature);
}

UMaterialInterface* ULightComponent::GetMaterial(int32 ElementIndex) const
{
	if (ElementIndex == 0)
	{
		return LightFunctionMaterial;
	}
	else
	{
		return NULL;
	}
}

void ULightComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* InMaterial)
{
	if (ElementIndex == 0)
	{
#if WITH_EDITOR
		StashedLightFunctionMaterial = nullptr;
#endif
		LightFunctionMaterial = InMaterial;
		MarkRenderStateDirty();
	}
}

void ULightComponent::PushSelectionToProxy()
{
	if (SceneProxy)
	{
		const bool bIsSelected = IsSelected() || IsOwnerSelected();
		FLightSceneProxy* LocalSceneProxy = SceneProxy;
		ENQUEUE_RENDER_COMMAND(SetLightSelection)(
			[LocalSceneProxy, bIsSelected](FRHICommandListImmediate& RHICmdList)
			{
				// NOTE: The selection flag is currently used on the C++ side for debug features, so simply updating this flag is enough.
				LocalSceneProxy->SetSelected(bIsSelected);
			});
	}

}

void ULightComponent::BuildSceneProxyDesc(const ULightComponent& InLightComponent, FLightSceneProxyDesc& OutProxyDesc)
{
	const FLightComponentMapBuildData* MapBuildData = InLightComponent.GetLightComponentMapBuildData();

	OutProxyDesc.LightComponent = &InLightComponent;
	OutProxyDesc.LightComponentId = InLightComponent.GetLightComponentId();
	OutProxyDesc.SceneInterface = InLightComponent.GetScene();
#if ACTOR_HAS_LABELS
	OutProxyDesc.OwnerNameOrLabel = InLightComponent.GetOwner() ? InLightComponent.GetOwner()->GetActorNameOrLabel() : InLightComponent.GetName();
#endif
	OutProxyDesc.bSelected = InLightComponent.IsSelected() || InLightComponent.IsOwnerSelected();
	OutProxyDesc.bContactShadowLengthInWS = InLightComponent.GetContactShadowLengthInWS();
	OutProxyDesc.bMovable = InLightComponent.IsMovable();
	OutProxyDesc.bStaticLighting = InLightComponent.HasStaticLighting();
	OutProxyDesc.bStaticShadowing = InLightComponent.HasStaticShadowing();
	OutProxyDesc.bCastDynamicShadow = InLightComponent.CastsDynamicShadow();
	OutProxyDesc.bCastStaticShadow = InLightComponent.CastsStaticShadow();
	OutProxyDesc.bCastTranslucentShadows = InLightComponent.CastTranslucentShadows;
	OutProxyDesc.bTransmission = InLightComponent.TransmitsThroughSurfaces();
	OutProxyDesc.bCastVolumetricShadow = InLightComponent.bCastVolumetricShadow;
	OutProxyDesc.bCastHairStrandsDeepShadow = InLightComponent.bCastDeepShadow;
	OutProxyDesc.bCastShadowsFromCinematicObjectsOnly = InLightComponent.bCastShadowsFromCinematicObjectsOnly;
	OutProxyDesc.bForceCachedShadowsForMovablePrimitives = InLightComponent.bForceCachedShadowsForMovablePrimitives;
	OutProxyDesc.bAffectReflection = InLightComponent.bAffectReflection;
	OutProxyDesc.bAffectGlobalIllumination = InLightComponent.bAffectGlobalIllumination;
	OutProxyDesc.bAffectTranslucentLighting = InLightComponent.bAffectTranslucentLighting;
	OutProxyDesc.bUsedAsAtmosphereSunLight = InLightComponent.IsUsedAsAtmosphereSunLight();
	OutProxyDesc.bUseRayTracedDistanceFieldShadows = InLightComponent.bUseRayTracedDistanceFieldShadows;
	OutProxyDesc.bAllowMegaLights = InLightComponent.bAllowMegaLights;
	OutProxyDesc.bIsPrecomputedLightingValid = InLightComponent.IsPrecomputedLightingValid();
	OutProxyDesc.bEnableLightShaftBloom = InLightComponent.bEnableLightShaftBloom;
	OutProxyDesc.Color = InLightComponent.GetColoredLightBrightness();
	OutProxyDesc.IndirectLightingScale = InLightComponent.IndirectLightingIntensity;
	OutProxyDesc.VolumetricScatteringIntensity = FMath::Max(InLightComponent.VolumetricScatteringIntensity, 0.0f);
	OutProxyDesc.RayEndBias = InLightComponent.GetRayEndBias();
	OutProxyDesc.ShadowResolutionScale = InLightComponent.ShadowResolutionScale;
	OutProxyDesc.ShadowBias = InLightComponent.ShadowBias;
	OutProxyDesc.ShadowSlopeBias = InLightComponent.ShadowSlopeBias;
	OutProxyDesc.ShadowSharpen = InLightComponent.ShadowSharpen;
	OutProxyDesc.ContactShadowLength = InLightComponent.ContactShadowLength;
	OutProxyDesc.ContactShadowCastingIntensity = InLightComponent.ContactShadowCastingIntensity;
	OutProxyDesc.ContactShadowNonCastingIntensity = InLightComponent.ContactShadowNonCastingIntensity;
	OutProxyDesc.SpecularScale = InLightComponent.SpecularScale;
	OutProxyDesc.DiffuseScale = InLightComponent.DiffuseScale;
	OutProxyDesc.LightGuid = InLightComponent.LightGuid;
	OutProxyDesc.MapBuildDataShadowMapChannel = MapBuildData ? MapBuildData->ShadowMapChannel : INDEX_NONE;
	OutProxyDesc.PreviewShadowMapChannel = InLightComponent.PreviewShadowMapChannel;
	OutProxyDesc.RayStartOffsetDepthScale = InLightComponent.RayStartOffsetDepthScale;
	OutProxyDesc.LightFunctionScale = InLightComponent.LightFunctionScale;
	OutProxyDesc.LightFunctionFadeDistance = InLightComponent.LightFunctionFadeDistance;
	OutProxyDesc.LightFunctionDisabledBrightness = InLightComponent.DisabledBrightness;
	OutProxyDesc.LightFunctionMaterial = InLightComponent.LightFunctionMaterial;
	OutProxyDesc.IESTexture = InLightComponent.IESTexture;
	OutProxyDesc.CastRaytracedShadow = InLightComponent.GetCastRaytracedShadow();
	OutProxyDesc.BloomScale = InLightComponent.BloomScale;
	OutProxyDesc.BloomThreshold = InLightComponent.BloomThreshold;
	OutProxyDesc.BloomMaxBrightness = InLightComponent.BloomMaxBrightness;
	OutProxyDesc.BloomTint = InLightComponent.BloomTint;
	OutProxyDesc.MegaLightsShadowMethod = InLightComponent.MegaLightsShadowMethod;
	OutProxyDesc.AtmosphereSunLightIndex = InLightComponent.GetAtmosphereSunLightIndex();
	OutProxyDesc.AtmosphereSunDiskColorScale = InLightComponent.GetAtmosphereSunDiskColorScale();
	OutProxyDesc.LightType = InLightComponent.GetLightType();
	OutProxyDesc.LightingChannelMask = GetLightingChannelMaskForStruct(InLightComponent.LightingChannels);
	OutProxyDesc.ViewLightingChannelMask = InLightComponent.ViewLightingChannels.GetMaskForStruct();
	OutProxyDesc.ComponentName = InLightComponent.GetFName();
	OutProxyDesc.LevelName = InLightComponent.GetOwner() ? InLightComponent.GetOwner()->GetLevel()->GetOutermost()->GetFName() : NAME_None;
	OutProxyDesc.SamplesPerPixel = InLightComponent.SamplesPerPixel;
	OutProxyDesc.DeepShadowLayerDistribution = InLightComponent.DeepShadowLayerDistribution;
}

/** Stores a light and a channel it has been assigned to. */
struct FLightAndChannel
{
	ULightComponent* Light;
	int32 Channel;

	FLightAndChannel(ULightComponent* InLight) :
		Light(InLight),
		Channel(INDEX_NONE)
	{}
};

struct FCompareLightsByArrayCount
{
	FORCEINLINE bool operator()( const TArray<FLightAndChannel*>& A, const TArray<FLightAndChannel*>& B ) const 
	{ 
		// Sort by descending array count
		return B.Num() < A.Num(); 
	}
};

/**
 * This function is supposed to be called only when
 * - loading a map (UEditorEngine::Map_Load)
 * - a light's lighting cache gets invalidated (ULightComponent::InvalidateLightingCacheDetailed)
 * - finishing a lighting build
 * If you're adding more call sites to this function, make sure not to break GPULightmass as it is based on the above assumption
 */

#if WITH_EDITOR
void ULightComponent::ReassignStationaryLightChannels(UWorld* TargetWorld, bool bAssignForLightingBuild, FStaticLightingBuildContext* LightingContext)
{
	TMap<FLightAndChannel*, TArray<FLightAndChannel*> > LightToOverlapMap;

	// Build an array of all static shadowing lights that need to be assigned
	for (TObjectIterator<ULightComponent> LightIt(RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); LightIt; ++LightIt)
	{
		ULightComponent* const LightComponent = *LightIt;
		AActor* LightOwner = LightComponent->GetOwner();

		const bool bLightIsInWorld = IsValid(LightOwner) && TargetWorld->ContainsActor(LightOwner);

		if (bLightIsInWorld 
			// Only operate on stationary light components (static shadowing only)
			&& LightComponent->HasStaticShadowing()
			&& !LightComponent->HasStaticLighting())
		{
			if (!LightingContext || LightingContext->ShouldIncludeActor(LightOwner))
			{				
				if (LightComponent->bAffectsWorld
					&& (LightComponent->CastShadows || LightComponent->LightFunctionMaterial)
					&& LightComponent->CastStaticShadows)
				{
					LightToOverlapMap.Add(new FLightAndChannel(LightComponent), TArray<FLightAndChannel*>());
				}
				else
				{
					// Reset the preview channel of stationary light components that shouldn't get a channel
					// This is necessary to handle a light being newly disabled
					LightComponent->PreviewShadowMapChannel = INDEX_NONE;

#if WITH_EDITOR
					LightComponent->UpdateLightSpriteTexture();
#endif
				}
			}
		}
	}

	// Build an array of overlapping lights
	for (TMap<FLightAndChannel*, TArray<FLightAndChannel*> >::TIterator It(LightToOverlapMap); It; ++It)
	{
		ULightComponent* CurrentLight = It.Key()->Light;
		TArray<FLightAndChannel*>& OverlappingLights = It.Value();

		if (bAssignForLightingBuild)
		{
			UMapBuildDataRegistry* Registry = LightingContext->GetOrCreateRegistryForActor(CurrentLight->GetOwner());
			FLightComponentMapBuildData& LightBuildData = Registry->FindOrAllocateLightBuildData(CurrentLight->LightGuid, true);
			LightBuildData.ShadowMapChannel = INDEX_NONE;
		}

		for (TMap<FLightAndChannel*, TArray<FLightAndChannel*> >::TIterator OtherIt(LightToOverlapMap); OtherIt; ++OtherIt)
		{
			ULightComponent* OtherLight = OtherIt.Key()->Light;

			if (CurrentLight != OtherLight 
				// Testing both directions because the spotlight <-> spotlight test is just cone vs bounding sphere
				//@todo - more accurate spotlight <-> spotlight intersection
				&& CurrentLight->AffectsBounds(FBoxSphereBounds(OtherLight->GetBoundingSphere()))
				&& OtherLight->AffectsBounds(FBoxSphereBounds(CurrentLight->GetBoundingSphere())))
			{
				OverlappingLights.Add(OtherIt.Key());
			}
		}
	}
		
	// Sort lights with the most overlapping lights first
	LightToOverlapMap.ValueSort(FCompareLightsByArrayCount());

	TMap<FLightAndChannel*, TArray<FLightAndChannel*> > SortedLightToOverlapMap;

	// Add directional lights to the beginning so they always get channels
	for (TMap<FLightAndChannel*, TArray<FLightAndChannel*> >::TIterator It(LightToOverlapMap); It; ++It)
	{
		FLightAndChannel* CurrentLight = It.Key();

		if (CurrentLight->Light->GetLightType() == LightType_Directional)
		{
			SortedLightToOverlapMap.Add(It.Key(), It.Value());
		}
	}

	// Add everything else, which has been sorted by descending overlaps
	for (TMap<FLightAndChannel*, TArray<FLightAndChannel*> >::TIterator It(LightToOverlapMap); It; ++It)
	{
		FLightAndChannel* CurrentLight = It.Key();

		if (CurrentLight->Light->GetLightType() != LightType_Directional)
		{
			SortedLightToOverlapMap.Add(It.Key(), It.Value());
		}
	}

	// Go through lights and assign shadowmap channels
	//@todo - retry with different ordering heuristics when it fails
	for (TMap<FLightAndChannel*, TArray<FLightAndChannel*> >::TIterator It(SortedLightToOverlapMap); It; ++It)
	{
		bool bChannelUsed[4] = {0};
		FLightAndChannel* CurrentLight = It.Key();
		const TArray<FLightAndChannel*>& OverlappingLights = It.Value();

		// Mark which channels have already been assigned to overlapping lights
		for (int32 OverlappingIndex = 0; OverlappingIndex < OverlappingLights.Num(); OverlappingIndex++)
		{
			FLightAndChannel* OverlappingLight = OverlappingLights[OverlappingIndex];

			if (OverlappingLight->Channel != INDEX_NONE)
			{
				bChannelUsed[OverlappingLight->Channel] = true;
			}
		}

		// Use the lowest free channel
		for (int32 ChannelIndex = 0; ChannelIndex < UE_ARRAY_COUNT(bChannelUsed); ChannelIndex++)
		{
			if (!bChannelUsed[ChannelIndex])
			{
				CurrentLight->Channel = ChannelIndex;
				break;
			}
		}
	}

	// Go through the assigned lights and update their render state and icon
	for (TMap<FLightAndChannel*, TArray<FLightAndChannel*> >::TIterator It(SortedLightToOverlapMap); It; ++It)
	{
		FLightAndChannel* CurrentLight = It.Key();

		if (CurrentLight->Light->PreviewShadowMapChannel != CurrentLight->Channel)
		{
			CurrentLight->Light->PreviewShadowMapChannel = CurrentLight->Channel;
#if WITH_EDITOR
			FStaticLightingSystemInterface::OnStationaryLightChannelReassigned.Broadcast(CurrentLight->Light, CurrentLight->Light->PreviewShadowMapChannel);
#endif
			CurrentLight->Light->MarkRenderStateDirty();
		}

#if WITH_EDITOR
		CurrentLight->Light->UpdateLightSpriteTexture();
#endif

		if (bAssignForLightingBuild)
		{
			UMapBuildDataRegistry* Registry = LightingContext->GetOrCreateRegistryForActor(CurrentLight->Light->GetOwner());
			FLightComponentMapBuildData& LightBuildData = Registry->FindOrAllocateLightBuildData(CurrentLight->Light->LightGuid, true);
			LightBuildData.ShadowMapChannel = CurrentLight->Channel;

			if (CurrentLight->Channel == INDEX_NONE)
			{
				FMessageLog("LightingResults").PerformanceWarning()
					->AddToken(FUObjectToken::Create(CurrentLight->Light->GetOwner()))
					->AddToken(FTextToken::Create( NSLOCTEXT("Lightmass", "LightmassError_FailedToAllocateShadowmapChannel", "Severe performance loss: Failed to allocate shadowmap channel for stationary light due to overlap - light will fall back to dynamic shadows!") ) );
			}
		}

		delete CurrentLight;
	}
}
#endif

static void ToggleLight(const TArray<FString>& Args)
{
	for (TObjectIterator<ULightComponent> It; It; ++It)
	{
		ULightComponent* Light = *It;
		if (Light->Mobility != EComponentMobility::Static)
		{
			FString LightName = (Light->GetOwner() ? Light->GetOwner()->GetFName() : Light->GetFName()).ToString();
			for (int32 ArgIndex = 0; ArgIndex < Args.Num(); ++ArgIndex)
			{
				const FString& ToggleName = Args[ArgIndex];
				if (LightName.Contains(ToggleName) )
				{
					Light->ToggleVisibility(/*bPropagateToChildren=*/ false);
					UE_LOGF(LogConsoleResponse,Display,"Now%lsvisible: %ls",
						Light->IsVisible() ? TEXT("") : TEXT(" not "),
						*Light->GetFullName()
						);
					break;
				}
			}
		}
	}
}

static FAutoConsoleCommand ToggleLightCmd(
	TEXT("ToggleLight"),
	TEXT("Toggles all lights whose name contains the specified string"),
	FConsoleCommandWithArgsDelegate::CreateStatic(ToggleLight),
	ECVF_Cheat
	);


#undef LOCTEXT_NAMESPACE
