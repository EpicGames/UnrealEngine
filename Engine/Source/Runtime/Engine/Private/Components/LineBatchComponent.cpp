// Copyright Epic Games, Inc. All Rights Reserved.

/*-----------------------------------------------------------------------------
	ULineBatchComponent implementation.
-----------------------------------------------------------------------------*/

#include "Components/LineBatchComponent.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/Engine.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MeshElementCollector.h"
#include "Engine/CollisionProfile.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMeshBuilder.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LineBatchComponent)

/** Represents a LineBatchComponent to the scene manager. */
class ENGINE_API FLineBatcherSceneProxy : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override;

	FLineBatcherSceneProxy(const ULineBatchComponent* InComponent);

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	/**
	*	Returns a struct that describes to the renderer when to draw this proxy.
	*	@param		View view to use to determine our relevance.
	*	@return		View relevance struct
	*/
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual uint32 GetMemoryFootprint( void ) const override;
	uint32 GetAllocatedSize( void ) const;

private:
	TArray<FBatchedLine> Lines;
	TArray<FBatchedPoint> Points;
	TArray<FBatchedMesh> Meshes;
};


FLineBatcherSceneProxy::FLineBatcherSceneProxy(const ULineBatchComponent* InComponent) :
	FPrimitiveSceneProxy(InComponent), Lines(InComponent->BatchedLines), 
	Points(InComponent->BatchedPoints), Meshes(InComponent->BatchedMeshes)
{
	bWillEverBeLit = false;
}

SIZE_T FLineBatcherSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FLineBatcherSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER( STAT_LineBatcherSceneProxy_GetDynamicMeshElements );

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];

#if UE_ENABLE_DEBUG_DRAWING
			//If we are in a debug draw build, this line batcher should write out to the debug only PDI
			FPrimitiveDrawInterface* PDI = Collector.GetDebugPDI(ViewIndex);
#else
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
#endif
			for (const FBatchedLine& Line: Lines)
			{
				PDI->DrawTranslucentLine(Line.Start,Line.End, Line.Color, Line.DepthPriority, Line.Thickness);
			}

			for (const FBatchedPoint& Point: Points)
			{
				PDI->DrawPoint(Point.Position, Point.Color, Point.PointSize, Point.DepthPriority);
			}

			for (const FBatchedMesh& Mesh: Meshes)
			{
				static FVector const PosX(1.f,0,0);
				static FVector const PosY(0,1.f,0);
				static FVector const PosZ(0,0,1.f);

				// this seems far from optimal in terms of perf, but it's for debugging
				FDynamicMeshBuilder MeshBuilder(View->GetFeatureLevel());

				// set up geometry
				for (const FVector& Vert: Mesh.MeshVerts)
				{
					MeshBuilder.AddVertex((FVector3f)Vert, FVector2f::ZeroVector, (FVector3f)PosX, (FVector3f)PosY, (FVector3f)PosZ, FColor::White );
				}
				//MeshBuilder.AddTriangles(M.MeshIndices);
				for (int32 Idx=0; Idx < Mesh.MeshIndices.Num(); Idx+=3)
				{
					MeshBuilder.AddTriangle( Mesh.MeshIndices[Idx], Mesh.MeshIndices[Idx+1], Mesh.MeshIndices[Idx+2] );
				}

				FMaterialRenderProxy* const MaterialRenderProxy = new FColoredMaterialRenderProxy(GEngine->DebugMeshMaterial->GetRenderProxy(), Mesh.Color);
				Collector.RegisterOneFrameMaterialProxy(MaterialRenderProxy);

				MeshBuilder.GetMesh(FMatrix::Identity, MaterialRenderProxy, Mesh.DepthPriority, false, false, ViewIndex, Collector);
			}
		}
	}
}

/**
*	Returns a struct that describes to the renderer when to draw this proxy.
*	@param		View view to use to determine our relevance.
*	@return		View relevance struct
*/
FPrimitiveViewRelevance FLineBatcherSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance ViewRelevance;
	ViewRelevance.bDrawRelevance = IsShown(View);
	ViewRelevance.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	ViewRelevance.bSeparateTranslucency = ViewRelevance.bNormalTranslucency = true;
	return ViewRelevance;
}

uint32 FLineBatcherSceneProxy::GetMemoryFootprint( void ) const
{
	return( sizeof( *this ) + GetAllocatedSize() );
}

uint32 FLineBatcherSceneProxy::GetAllocatedSize( void ) const 
{ 
	return( FPrimitiveSceneProxy::GetAllocatedSize() + Lines.GetAllocatedSize() + Points.GetAllocatedSize() + Meshes.GetAllocatedSize() ); 
}

ULineBatchComponent::ULineBatchComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	bUseEditorCompositing = true;
	SetGenerateOverlapEvents(false);
	bCalculateAccurateBounds = true;
	DefaultLifeTime = 1.0f;

	// Ignore streaming updates since GetUsedMaterials() is not implemented.
	SetIgnoreStreamingManagerUpdate(true);
}

void ULineBatchComponent::DrawLine(const FVector& Start, const FVector& End, const FLinearColor& Color, uint8 DepthPriority, const float Thickness, const float LifeTime, uint32 BatchID)
{
	BatchedLines.Emplace(Start, End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	// LineBatcher and PersistentLineBatcher components will be updated at the end of UWorld::Tick
	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawLines(TArrayView<FBatchedLine> InLines)
{
	BatchedLines.Append(InLines);

	// LineBatcher and PersistentLineBatcher components will be updated at the end of UWorld::Tick
	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawPoint(
	const FVector& Position,
	const FLinearColor& Color,
	float PointSize,
	uint8 DepthPriority,
	float LifeTime,
	uint32 BatchID
	)
{
	BatchedPoints.Emplace(Position, Color, PointSize, LifeTime, DepthPriority, BatchID);
	// LineBatcher and PersistentLineBatcher components will be updated at the end of UWorld::Tick
	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawBox(const FBox& Box, const FMatrix& TM, FLinearColor Color, uint8 InDepthPriorityGroup, uint32 BatchID)
{
	FVector	B[2], P, Q;
	B[0] = Box.Min;
	B[1] = Box.Max;

	for( int32 ai=0; ai<2; ai++ ) for( int32 aj=0; aj<2; aj++ )
	{
		P.X=B[ai].X; Q.X=B[ai].X;
		P.Y=B[aj].Y; Q.Y=B[aj].Y;
		P.Z=B[0].Z; Q.Z=B[1].Z;
		BatchedLines.Emplace(TM.TransformPosition(P), TM.TransformPosition(Q), Color, DefaultLifeTime, 0.0f, InDepthPriorityGroup, BatchID);

		P.Y=B[ai].Y; Q.Y=B[ai].Y;
		P.Z=B[aj].Z; Q.Z=B[aj].Z;
		P.X=B[0].X; Q.X=B[1].X;
		BatchedLines.Emplace(TM.TransformPosition(P), TM.TransformPosition(Q), Color, DefaultLifeTime, 0.0f, InDepthPriorityGroup, BatchID);

		P.Z=B[ai].Z; Q.Z=B[ai].Z;
		P.X=B[aj].X; Q.X=B[aj].X;
		P.Y=B[0].Y; Q.Y=B[1].Y;
		BatchedLines.Emplace(TM.TransformPosition(P), TM.TransformPosition(Q), Color, DefaultLifeTime, 0.0f, InDepthPriorityGroup, BatchID);
	}

	// LineBatcher and PersistentLineBatcher components will be updated at the end of UWorld::Tick
	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawBox(FVector const& Center, FVector const& Box, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID)
{
	BatchedLines.Emplace(Center + FVector( Box.X,  Box.Y,  Box.Z), Center + FVector( Box.X, -Box.Y, Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector( Box.X, -Box.Y,  Box.Z), Center + FVector(-Box.X, -Box.Y, Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector(-Box.X, -Box.Y,  Box.Z), Center + FVector(-Box.X,  Box.Y, Box.Z) ,Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector(-Box.X,  Box.Y,  Box.Z), Center + FVector( Box.X,  Box.Y, Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);

	BatchedLines.Emplace(Center + FVector( Box.X,  Box.Y, -Box.Z), Center + FVector( Box.X, -Box.Y, -Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector( Box.X, -Box.Y, -Box.Z), Center + FVector(-Box.X, -Box.Y, -Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector(-Box.X, -Box.Y, -Box.Z), Center + FVector(-Box.X,  Box.Y, -Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector(-Box.X,  Box.Y, -Box.Z), Center + FVector( Box.X,  Box.Y, -Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);

	BatchedLines.Emplace(Center + FVector( Box.X,  Box.Y,  Box.Z), Center + FVector( Box.X,  Box.Y, -Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector( Box.X, -Box.Y,  Box.Z), Center + FVector( Box.X, -Box.Y, -Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector(-Box.X, -Box.Y,  Box.Z), Center + FVector(-Box.X, -Box.Y, -Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(Center + FVector(-Box.X,  Box.Y,  Box.Z), Center + FVector(-Box.X,  Box.Y, -Box.Z), Color, LifeTime, Thickness, DepthPriority, BatchID);

	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawBox(FVector const& Center, FVector const& Box, const FQuat& Rotation, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID)
{
	FTransform const Transform(Rotation);
	FVector Start = Transform.TransformPosition(FVector( Box.X,  Box.Y,  Box.Z));
	FVector End = Transform.TransformPosition(FVector( Box.X, -Box.Y, Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector( Box.X, -Box.Y,  Box.Z));
	End = Transform.TransformPosition(FVector(-Box.X, -Box.Y, Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector(-Box.X, -Box.Y,  Box.Z));
	End = Transform.TransformPosition(FVector(-Box.X,  Box.Y, Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector(-Box.X,  Box.Y,  Box.Z));
	End = Transform.TransformPosition(FVector( Box.X,  Box.Y, Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector( Box.X,  Box.Y, -Box.Z));
	End = Transform.TransformPosition(FVector( Box.X, -Box.Y, -Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector( Box.X, -Box.Y, -Box.Z));
	End = Transform.TransformPosition(FVector(-Box.X, -Box.Y, -Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector(-Box.X, -Box.Y, -Box.Z));
	End = Transform.TransformPosition(FVector(-Box.X,  Box.Y, -Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector(-Box.X,  Box.Y, -Box.Z));
	End = Transform.TransformPosition(FVector( Box.X,  Box.Y, -Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector( Box.X,  Box.Y,  Box.Z));
	End = Transform.TransformPosition(FVector( Box.X,  Box.Y, -Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector( Box.X, -Box.Y,  Box.Z));
	End = Transform.TransformPosition(FVector( Box.X, -Box.Y, -Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector(-Box.X, -Box.Y,  Box.Z));
	End = Transform.TransformPosition(FVector(-Box.X, -Box.Y, -Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	Start = Transform.TransformPosition(FVector(-Box.X,  Box.Y,  Box.Z));
	End = Transform.TransformPosition(FVector(-Box.X,  Box.Y, -Box.Z));
	BatchedLines.Emplace(Center + Start, Center + End, Color, LifeTime, Thickness, DepthPriority, BatchID);

	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawSolidBox(const FBox& Box, const FTransform& Xform, const FColor& Color, uint8 DepthPriority, float LifeTime, uint32 BatchID)
{
	int32 const NewMeshIdx = BatchedMeshes.Add(FBatchedMesh());
	FBatchedMesh& BM = BatchedMeshes[NewMeshIdx];

	BM.Color = Color;
	BM.DepthPriority = DepthPriority;
	BM.RemainingLifeTime = LifeTime;
	BM.BatchID = BatchID;

	BM.MeshVerts.AddUninitialized(8);
	BM.MeshVerts[0] = Xform.TransformPosition( FVector(Box.Min.X, Box.Min.Y, Box.Max.Z) );
	BM.MeshVerts[1] = Xform.TransformPosition( FVector(Box.Max.X, Box.Min.Y, Box.Max.Z) );
	BM.MeshVerts[2] = Xform.TransformPosition( FVector(Box.Min.X, Box.Min.Y, Box.Min.Z) );
	BM.MeshVerts[3] = Xform.TransformPosition( FVector(Box.Max.X, Box.Min.Y, Box.Min.Z) );
	BM.MeshVerts[4] = Xform.TransformPosition( FVector(Box.Min.X, Box.Max.Y, Box.Max.Z) );
	BM.MeshVerts[5] = Xform.TransformPosition( FVector(Box.Max.X, Box.Max.Y, Box.Max.Z) );
	BM.MeshVerts[6] = Xform.TransformPosition( FVector(Box.Min.X, Box.Max.Y, Box.Min.Z) );
	BM.MeshVerts[7] = Xform.TransformPosition( FVector(Box.Max.X, Box.Max.Y, Box.Min.Z) );

	// clockwise
	BM.MeshIndices.AddUninitialized(36);
	constexpr int32 Indices[36] = {	3,2,0,
		3,0,1,
		7,3,1,
		7,1,5,
		6,7,5,
		6,5,4,
		2,6,4,
		2,4,0,
		1,0,4,
		1,4,5,
		7,6,2,
		7,2,3	};

	for (int32 Idx=0; Idx<36; ++Idx)
	{
		BM.MeshIndices[Idx] = Indices[Idx];
	}

	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawMesh(TArray<FVector> const& Verts, TArray<int32> const& Indices, FColor const& Color, uint8 DepthPriority, float LifeTime, uint32 BatchID)
{
	// modifying array element directly to avoid copying arrays
	int32 const NewMeshIdx = BatchedMeshes.Add(FBatchedMesh());
	FBatchedMesh& BM = BatchedMeshes[NewMeshIdx];

	BM.MeshIndices = Indices;
	BM.MeshVerts = Verts;
	BM.Color = Color;
	BM.DepthPriority = DepthPriority;
	BM.RemainingLifeTime = LifeTime;
	BM.BatchID = BatchID;

	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawDirectionalArrow(const FMatrix& ArrowToWorld, FLinearColor InColor, float Length, float ArrowSize, uint8 DepthPriority, uint32 BatchID)
{
	const FVector Tip = ArrowToWorld.TransformPosition(FVector(Length,0,0));
	BatchedLines.Emplace(Tip,ArrowToWorld.TransformPosition(FVector::ZeroVector),InColor,DefaultLifeTime,0.0f,DepthPriority, BatchID);
	BatchedLines.Emplace(Tip,ArrowToWorld.TransformPosition(FVector(Length-ArrowSize,+ArrowSize,+ArrowSize)),InColor,DefaultLifeTime,0.0f,DepthPriority, BatchID);
	BatchedLines.Emplace(Tip,ArrowToWorld.TransformPosition(FVector(Length-ArrowSize,+ArrowSize,-ArrowSize)),InColor,DefaultLifeTime,0.0f,DepthPriority, BatchID);
	BatchedLines.Emplace(Tip,ArrowToWorld.TransformPosition(FVector(Length-ArrowSize,-ArrowSize,+ArrowSize)),InColor,DefaultLifeTime,0.0f,DepthPriority, BatchID);
	BatchedLines.Emplace(Tip,ArrowToWorld.TransformPosition(FVector(Length-ArrowSize,-ArrowSize,-ArrowSize)),InColor,DefaultLifeTime,0.0f,DepthPriority, BatchID);

	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawDirectionalArrow(FVector const& LineStart, FVector const& LineEnd, float ArrowSize, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID)
{
	FVector Dir = (LineEnd-LineStart);
	Dir.Normalize();
	FVector Up(0, 0, 1);
	FVector Right = Dir ^ Up;
	if (!Right.IsNormalized())
	{
		Dir.FindBestAxisVectors(Up, Right);
	}
	const FVector Origin = FVector::ZeroVector;
	FMatrix TM;
	// get matrix with dir/right/up
	TM.SetAxes(&Dir, &Right, &Up, &Origin);

	// since dir is x direction, my arrow will be pointing +y, -x and -y, -x
	const float ArrowSqrt = FMath::Sqrt(ArrowSize);

	BatchedLines.Emplace(LineStart,LineEnd, Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(LineEnd,LineEnd + TM.TransformPosition(FVector(-ArrowSqrt, ArrowSqrt, 0)), Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(LineEnd,LineEnd + TM.TransformPosition(FVector(-ArrowSqrt, -ArrowSqrt, 0)), Color, LifeTime, Thickness, DepthPriority, BatchID);

	MarkRenderStateDirty();
}


void ULineBatchComponent::AddHalfCircle(const FVector& Base, const FVector& X, const FVector& Y, const FLinearColor& Color, const float Radius, int32 NumSides, const float LifeTime, uint8 DepthPriority, const float Thickness, const uint32 BatchID)
{
	// Need at least 2 sides
	NumSides = FMath::Max(NumSides, 2);
	const float AngleDelta = 2.0f * UE_PI / NumSides;
	FVector	LastVertex = Base + X * Radius;

	for( int32 SideIndex = 0; SideIndex < (NumSides/2); SideIndex++)
	{
		FVector	Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		BatchedLines.Emplace(LastVertex, Vertex, Color, LifeTime, Thickness, DepthPriority, BatchID);
		LastVertex = Vertex;
	}
}

void ULineBatchComponent::AddCircle(const FVector& Base, const FVector& X, const FVector& Y, const FLinearColor& Color, const float Radius, int32 NumSides, const float LifeTime, uint8 DepthPriority, const float Thickness, const uint32 BatchID)
{
	// Need at least 2 sides
	NumSides = FMath::Max(NumSides, 2);
	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	FVector	LastVertex = Base + X * Radius;

	for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
	{
		const FVector Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		BatchedLines.Emplace(LastVertex, Vertex, Color, LifeTime, Thickness, DepthPriority, BatchID);
		LastVertex = Vertex;
	}
}

/** Draw a circle */
void ULineBatchComponent::DrawCircle(const FVector& Base, const FVector& X, const FVector& Y, FLinearColor Color, float Radius, int32 NumSides, uint8 DepthPriority, uint32 BatchID)
{
	AddCircle(Base, X, Y, Color, Radius, NumSides, DefaultLifeTime, DepthPriority, 0.f, BatchID);
		
	MarkRenderStateDirty();
}

/** Draw a sphere */
void ULineBatchComponent::DrawSphere(FVector const& Center, float Radius, int32 Segments, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID)
{
	// Need at least 4 segments
	Segments = FMath::Max(Segments, 4);

	const float AngleInc = 2.f * UE_PI / Segments;
	int32 NumSegmentsY = Segments;
	float Latitude = AngleInc;
	float SinY1 = 0.0f, CosY1 = 1.0f;

	while (NumSegmentsY--)
	{
		const float SinY2 = FMath::Sin(Latitude);
		const float CosY2 = FMath::Cos(Latitude);

		FVector Vertex1 = FVector(SinY1, 0.0f, CosY1) * Radius + Center;
		FVector Vertex3 = FVector(SinY2, 0.0f, CosY2) * Radius + Center;
		float Longitude = AngleInc;

		int32 NumSegmentsX = Segments;
		while (NumSegmentsX--)
		{
			const float SinX = FMath::Sin(Longitude);
			const float CosX = FMath::Cos(Longitude);

			const FVector Vertex2 = FVector((CosX * SinY1), (SinX * SinY1), CosY1) * Radius + Center;
			const FVector Vertex4 = FVector((CosX * SinY2), (SinX * SinY2), CosY2) * Radius + Center;

			BatchedLines.Emplace(Vertex1, Vertex2, Color, LifeTime, Thickness, DepthPriority, BatchID);
			BatchedLines.Emplace(Vertex1, Vertex3, Color, LifeTime, Thickness, DepthPriority, BatchID);

			Vertex1 = Vertex2;
			Vertex3 = Vertex4;
			Longitude += AngleInc;
		}
		SinY1 = SinY2;
		CosY1 = CosY2;
		Latitude += AngleInc;
	}

	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawCylinder(FVector const& Start, FVector const& End, float Radius, int32 Segments, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID)
{
	// Need at least 4 segments
	Segments = FMath::Max(Segments, 4);

	// Rotate a point around axis to form cylinder segments
	const float AngleInc = 360.f / Segments;
	float Angle = AngleInc;

	// Default for Axis is up
	FVector Axis = (End - Start).GetSafeNormal();
	if( Axis.IsZero() )
	{
		Axis = FVector(0.f, 0.f, 1.f);
	}

	FVector Perpendicular, Dummy;
	Axis.FindBestAxisVectors(Perpendicular, Dummy);
		
	FVector Segment = Perpendicular.RotateAngleAxis(0, Axis) * Radius;
	FVector P1 = Segment + Start;
	FVector P3 = Segment + End;

	while( Segments-- )
	{
		Segment = Perpendicular.RotateAngleAxis(Angle, Axis) * Radius;
		FVector P2 = Segment + Start;
		FVector P4 = Segment + End;

		BatchedLines.Emplace(P2, P4, Color, LifeTime, Thickness, DepthPriority, BatchID);
		BatchedLines.Emplace(P1, P2, Color, LifeTime, Thickness, DepthPriority, BatchID);
		BatchedLines.Emplace(P3, P4, Color, LifeTime, Thickness, DepthPriority, BatchID);

		P1 = P2;
		P3 = P4;
		Angle += AngleInc;
	}

	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawCone(FVector const& Origin, FVector const& Direction, float Length, float AngleWidth, float AngleHeight, int32 NumSides, FLinearColor DrawColor, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID)
{
	// Need at least 4 sides
	NumSides = FMath::Max(NumSides, 4);

	const float Angle1 = FMath::Clamp<float>(AngleHeight, UE_KINDA_SMALL_NUMBER, (UE_PI - UE_KINDA_SMALL_NUMBER));
	const float Angle2 = FMath::Clamp<float>(AngleWidth, UE_KINDA_SMALL_NUMBER, (UE_PI - UE_KINDA_SMALL_NUMBER));

	const float SinX_2 = FMath::Sin(0.5f * Angle1);
	const float SinY_2 = FMath::Sin(0.5f * Angle2);

	const float SinSqX_2 = SinX_2 * SinX_2;
	const float SinSqY_2 = SinY_2 * SinY_2;

	TArray<FVector> ConeVerts;
	ConeVerts.AddUninitialized(NumSides);

	for(int32 i = 0; i < NumSides; i++)
	{
		const float Fraction	= (float)i/(float)(NumSides);
		const float Thi			= 2.f * UE_PI * Fraction;
		const float Phi			= FMath::Atan2(FMath::Sin(Thi)*SinY_2, FMath::Cos(Thi)*SinX_2);
		const float SinPhi		= FMath::Sin(Phi);
		const float CosPhi		= FMath::Cos(Phi);
		const float SinSqPhi	= SinPhi*SinPhi;
		const float CosSqPhi	= CosPhi*CosPhi;

		const float RSq			= SinSqX_2*SinSqY_2 / (SinSqX_2*SinSqPhi + SinSqY_2*CosSqPhi);
		const float R			= FMath::Sqrt(RSq);
		const float Sqr			= FMath::Sqrt(1-RSq);
		const float Alpha		= R*CosPhi;
		const float Beta		= R*SinPhi;

		ConeVerts[i].X = (1 - 2*RSq);
		ConeVerts[i].Y = 2 * Sqr * Alpha;
		ConeVerts[i].Z = 2 * Sqr * Beta;
	}

	// Calculate transform for cone.
	FVector YAxis, ZAxis;
	const FVector DirectionNorm = Direction.GetSafeNormal();
	DirectionNorm.FindBestAxisVectors(YAxis, ZAxis);
	const FMatrix ConeToWorld = FScaleMatrix(FVector(Length)) * FMatrix(DirectionNorm, YAxis, ZAxis, Origin);

	FVector CurrentPoint, PrevPoint, FirstPoint;
	for(int32 i = 0; i < NumSides; i++)
	{
		CurrentPoint = ConeToWorld.TransformPosition(ConeVerts[i]);
		BatchedLines.Emplace(ConeToWorld.GetOrigin(), CurrentPoint, DrawColor, LifeTime, Thickness, DepthPriority, BatchID);

		// PrevPoint must be defined to draw junctions
		if( i > 0 )
		{
			BatchedLines.Emplace(PrevPoint, CurrentPoint, DrawColor, LifeTime, Thickness, DepthPriority, BatchID);
		}
		else
		{
			FirstPoint = CurrentPoint;
		}

		PrevPoint = CurrentPoint;
	}
	// Connect last junction to first
	BatchedLines.Emplace(CurrentPoint, FirstPoint, DrawColor, LifeTime, Thickness, DepthPriority, BatchID);

	MarkRenderStateDirty();
}

void ULineBatchComponent::DrawCapsule(FVector const& Center, float HalfHeight, float Radius, const FQuat& Rotation, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID)
{
	constexpr int32 DrawCollisionSides = 16;
	const FVector Origin = Center;
	const FMatrix Axes = FQuatRotationTranslationMatrix(Rotation, FVector::ZeroVector);
	const FVector XAxis = Axes.GetScaledAxis( EAxis::X );
	const FVector YAxis = Axes.GetScaledAxis( EAxis::Y );
	const FVector ZAxis = Axes.GetScaledAxis( EAxis::Z ); 

	// Draw top and bottom circles
	const float HalfAxis = FMath::Max<float>(HalfHeight - Radius, 1.f);
	const FVector TopEnd = Origin + HalfAxis*ZAxis;
	const FVector BottomEnd = Origin - HalfAxis*ZAxis;

	AddCircle(TopEnd, XAxis, YAxis, Color, Radius, DrawCollisionSides, LifeTime, DepthPriority, Thickness, BatchID);
	AddCircle( BottomEnd, XAxis, YAxis, Color, Radius, DrawCollisionSides, LifeTime, DepthPriority, Thickness, BatchID);

	// Draw domed caps
	AddHalfCircle( TopEnd, YAxis, ZAxis, Color, Radius, DrawCollisionSides, LifeTime, DepthPriority, Thickness, BatchID);
	AddHalfCircle( TopEnd, XAxis, ZAxis, Color, Radius, DrawCollisionSides, LifeTime, DepthPriority, Thickness, BatchID);

	const FVector NegZAxis = -ZAxis;

	AddHalfCircle( BottomEnd, YAxis, NegZAxis, Color, Radius, DrawCollisionSides, LifeTime, DepthPriority, Thickness, BatchID);
	AddHalfCircle( BottomEnd, XAxis, NegZAxis, Color, Radius, DrawCollisionSides, LifeTime, DepthPriority, Thickness, BatchID);

	// Draw connected lines
	BatchedLines.Emplace(TopEnd + Radius*XAxis, BottomEnd + Radius*XAxis, Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(TopEnd - Radius*XAxis, BottomEnd - Radius*XAxis, Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(TopEnd + Radius*YAxis, BottomEnd + Radius*YAxis, Color, LifeTime, Thickness, DepthPriority, BatchID);
	BatchedLines.Emplace(TopEnd - Radius*YAxis, BottomEnd - Radius*YAxis, Color, LifeTime, Thickness, DepthPriority, BatchID);

	MarkRenderStateDirty();
}

void ULineBatchComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	bool bDirty = false;
	// Update the life time of batched lines, removing the lines which have expired.
	for(int32 LineIndex=0; LineIndex < BatchedLines.Num(); LineIndex++)
	{
		FBatchedLine& Line = BatchedLines[LineIndex];
		if (Line.RemainingLifeTime > 0.0f)
		{
			Line.RemainingLifeTime -= DeltaTime;
			if(Line.RemainingLifeTime <= 0.0f)
			{
				// The line has expired, remove it.
				BatchedLines.RemoveAtSwap(LineIndex--);
				bDirty = true;
			}
		}
	}

	// Update the life time of batched points, removing the points which have expired.
	for(int32 PtIndex=0; PtIndex < BatchedPoints.Num(); PtIndex++)
	{
		FBatchedPoint& Pt = BatchedPoints[PtIndex];
		if (Pt.RemainingLifeTime > 0.0f)
		{
			Pt.RemainingLifeTime -= DeltaTime;
			if(Pt.RemainingLifeTime <= 0.0f)
			{
				// The point has expired, remove it.
				BatchedPoints.RemoveAtSwap(PtIndex--);
				bDirty = true;
			}
		}
	}

	// Update the life time of batched meshes, removing the meshes which have expired.
	for(int32 MeshIndex=0; MeshIndex < BatchedMeshes.Num(); MeshIndex++)
	{
		FBatchedMesh& Mesh = BatchedMeshes[MeshIndex];
		if (Mesh.RemainingLifeTime > 0.0f)
		{
			Mesh.RemainingLifeTime -= DeltaTime;
			if(Mesh.RemainingLifeTime <= 0.0f)
			{
				// The mesh has expired, remove it.
				BatchedMeshes.RemoveAtSwap(MeshIndex--);
				bDirty = true;
			}
		}
	}

	if(bDirty)
	{
		MarkRenderStateDirty();
	}
}

void ULineBatchComponent::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);

	bool bDirty = false;
	for (FBatchedLine& Line : BatchedLines)
	{
		Line.Start += InOffset;
		Line.End += InOffset;
		bDirty = true;
	}

	for (FBatchedPoint& Point : BatchedPoints)
	{
		Point.Position += InOffset;
		bDirty = true;
	}

	for (FBatchedMesh& Mesh : BatchedMeshes)
	{
		for (FVector& Vert : Mesh.MeshVerts)
		{
			Vert += InOffset;
			bDirty = true;
		}
	}

	if (bDirty)
	{
		MarkRenderStateDirty();
	}
}

/**
* Creates a new scene proxy for the line batcher component.
* @return	Pointer to the FLineBatcherSceneProxy
*/
FPrimitiveSceneProxy* ULineBatchComponent::CreateSceneProxy()
{
	return new FLineBatcherSceneProxy(this);
}

FBoxSphereBounds ULineBatchComponent::CalcBounds( const FTransform& LocalToWorld ) const 
{
	if (!bCalculateAccurateBounds)
	{
		const FVector BoxExtent(HALF_WORLD_MAX);
		return FBoxSphereBounds(FVector::ZeroVector, BoxExtent, BoxExtent.Size());
	}

	FBox BBox(ForceInit);
	for (const FBatchedLine& Line : BatchedLines)
	{
		BBox += Line.Start;
		BBox += Line.End;
	}

	for (const FBatchedPoint& Point : BatchedPoints)
	{
		BBox += Point.Position;
	}

	for (const FBatchedMesh& Mesh : BatchedMeshes)
	{
		for (const FVector& Vert : Mesh.MeshVerts)
		{
			BBox += Vert;
		}
	}

	if (BBox.IsValid)
	{
		// Points are in world space, so no need to transform.
		return FBoxSphereBounds(BBox);
	}
	else
	{
		const FVector BoxExtent(1.f);
		return FBoxSphereBounds(LocalToWorld.GetLocation(), BoxExtent, 1.f);
	}
}

void ULineBatchComponent::Flush()
{
	if (BatchedLines.Num() > 0 || BatchedPoints.Num() > 0 || BatchedMeshes.Num() > 0)
	{
		BatchedLines.Empty();
		BatchedPoints.Empty();
		BatchedMeshes.Empty();
		MarkRenderStateDirty();
	}
}

void ULineBatchComponent::ClearBatch(uint32 BatchID)
{
	if (BatchID == INVALID_ID)
	{
		return;
	}
	
	bool bDirty = BatchedLines.RemoveAllSwap([BatchID](const FBatchedLine& Element) { return Element.BatchID == BatchID; }) > 0;
	bDirty |= BatchedPoints.RemoveAllSwap([BatchID](const FBatchedPoint& Pt) { return Pt.BatchID == BatchID; }) > 0;
	bDirty |= BatchedMeshes.RemoveAllSwap([BatchID](const FBatchedMesh& Mesh) { return Mesh.BatchID == BatchID; }) > 0;

	if(bDirty)
	{
		MarkRenderStateDirty();
	}
}


