// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/LocalFogVolumeComponent.h"
#include "Engine/LocalFogVolume.h"
#include "LocalFogVolumeSceneProxy.h"
#include "Components/ArrowComponent.h"
#include "Components/DrawSphereComponent.h"
#include "Components/BillboardComponent.h"
#include "MeshElementCollector.h"
#include "SceneInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/World.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalFogVolumeComponent)


ULocalFogVolumeComponent::ULocalFogVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LocalFogVolumeSceneProxy(nullptr)
{
	Mobility = EComponentMobility::Movable;
}

ULocalFogVolumeComponent::~ULocalFogVolumeComponent()
{
}

void ULocalFogVolumeComponent::SendRenderTransformCommand()
{
	if (LocalFogVolumeSceneProxy)
	{
		FTransform ComponentTransform = GetComponentTransform();
		FLocalFogVolumeSceneProxy* SceneProxy = LocalFogVolumeSceneProxy;
		ENQUEUE_RENDER_COMMAND(FUpdateLocalFogVolumeSceneProxyTransformCommand)(
			[SceneProxy, ComponentTransform](FRHICommandList& RHICmdList)
			{
				// Nothing else is needed so that command could actually go.
				SceneProxy->UpdateComponentTransform(ComponentTransform);
			});
	}
}

void ULocalFogVolumeComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	bool bHidden = false;
#if WITH_EDITORONLY_DATA
	bHidden = GetOwner() ? GetOwner()->bHiddenEdLevel : false;
#endif // WITH_EDITORONLY_DATA
	if (!ShouldComponentAddToScene())
	{
		bHidden = true;
	}

	if (GetVisibleFlag() && !bHidden && ShouldComponentAddToScene() && ShouldRender() && IsRegistered() && (GetOuter() == NULL || !GetOuter()->HasAnyFlags(RF_ClassDefaultObject)))
	{
		LocalFogVolumeSceneProxy = CreateSceneProxy();
		GetWorld()->Scene->AddLocalFogVolume(LocalFogVolumeSceneProxy);
	}
}

void ULocalFogVolumeComponent::SendRenderTransform_Concurrent()
{
	Super::SendRenderTransform_Concurrent();
	SendRenderTransformCommand();
}

void ULocalFogVolumeComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	if (LocalFogVolumeSceneProxy)
	{
		GetWorld()->Scene->RemoveLocalFogVolume(LocalFogVolumeSceneProxy);

		FLocalFogVolumeSceneProxy* SceneProxy = LocalFogVolumeSceneProxy;
		ENQUEUE_RENDER_COMMAND(FDestroyLocalFogVolumeSceneProxyCommand)(
			[SceneProxy](FRHICommandList& RHICmdList)
			{
				delete SceneProxy;
			});

		LocalFogVolumeSceneProxy = nullptr;
	}
}

#define LFV_BLUEPRINT_SETFUNCTION(MemberType, MemberName) void ULocalFogVolumeComponent::Set##MemberName(MemberType NewValue)\
{\
	if (AreDynamicDataChangesAllowed() && MemberName != NewValue)\
	{\
		MemberName = FMath::Max(MemberType(0), NewValue);\
		MarkRenderStateDirty();\
	}\
}\

#define LFV_BLUEPRINT_SETFUNCTION_LINEARCOLOR(MemberName) void ULocalFogVolumeComponent::Set##MemberName(FLinearColor NewValue)\
{\
	if (AreDynamicDataChangesAllowed() && MemberName != NewValue)\
	{\
		MemberName = NewValue.GetClamped(0.0f, 1e38f); \
		MarkRenderStateDirty();\
	}\
}\

#define LFV_BLUEPRINT_SETFUNCTION_LINEARCOLOR01(MemberName) void ULocalFogVolumeComponent::Set##MemberName(FLinearColor NewValue)\
{\
	if (AreDynamicDataChangesAllowed() && MemberName != NewValue)\
	{\
		MemberName = NewValue.GetClamped(0.0f, 1.0f); \
		MarkRenderStateDirty();\
	}\
}\

LFV_BLUEPRINT_SETFUNCTION(float, RadialFogExtinction);
LFV_BLUEPRINT_SETFUNCTION(float, HeightFogExtinction);
LFV_BLUEPRINT_SETFUNCTION(float, HeightFogFalloff);
LFV_BLUEPRINT_SETFUNCTION(float, HeightFogOffset);
LFV_BLUEPRINT_SETFUNCTION(float, FogPhaseG);
LFV_BLUEPRINT_SETFUNCTION(float, FogStartDistance);
LFV_BLUEPRINT_SETFUNCTION_LINEARCOLOR01(FogAlbedo);
LFV_BLUEPRINT_SETFUNCTION_LINEARCOLOR(FogEmissive);


#if WITH_EDITOR

bool ULocalFogVolumeComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (HeightFogExtinction <= 0.0f)
		{
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULocalFogVolumeComponent, HeightFogFalloff)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULocalFogVolumeComponent, HeightFogOffset))
			{
				return false;
			}
		}
	}

	return true;
}

void ULocalFogVolumeComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	const FName PropertyName = Property ? Property->GetFName() : NAME_None;

	if (PropertyName == TEXT("RelativeLocation") ||
		PropertyName == TEXT("RelativeRotation") ||
		PropertyName == TEXT("RelativeScale3D"))
	{
		SendRenderTransformCommand();
	}
	else
	{
		MarkRenderStateDirty();
	}
}

#endif // WITH_EDITOR

FLocalFogVolumeSceneProxy* ULocalFogVolumeComponent::CreateSceneProxy()
{
	return new FLocalFogVolumeSceneProxy(this);
}

#if WITH_EDITOR
#include "ObjectEditorUtils.h"
#endif

ALocalFogVolume::ALocalFogVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<ULocalFogVolumeComponent>(TEXT("NewLocalFogVolumeComponent")))
{
	LocalFogVolumeVolume = CreateDefaultSubobject<ULocalFogVolumeComponent>(TEXT("LocalFogVolumeComponent"));
	RootComponent = LocalFogVolumeVolume;

#if WITH_EDITORONLY_DATA

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> LocalFogVolumeTextureObject;
			FName ID_LocalFogVolume;
			FText NAME_LocalFogVolume;
			FConstructorStatics()
				: LocalFogVolumeTextureObject(TEXT("/Engine/EditorResources/S_SkyAtmosphere"))	// TODO
				, ID_LocalFogVolume(TEXT("Fog"))
				, NAME_LocalFogVolume(NSLOCTEXT("SpriteCategory", "Fog", "Fog"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (GetSpriteComponent())
		{
			GetSpriteComponent()->Sprite = ConstructorStatics.LocalFogVolumeTextureObject.Get();
			GetSpriteComponent()->SetRelativeScale3D(FVector(1.0f, 1.0f, 1.0f));
			GetSpriteComponent()->SpriteInfo.Category = ConstructorStatics.ID_LocalFogVolume;
			GetSpriteComponent()->SpriteInfo.DisplayName = ConstructorStatics.NAME_LocalFogVolume;
			GetSpriteComponent()->SetupAttachment(LocalFogVolumeVolume);
		}
	}
#endif // WITH_EDITORONLY_DATA

	PrimaryActorTick.bCanEverTick = false;

	SetHidden(false);
}
