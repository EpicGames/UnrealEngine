// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/LocalLightComponent.h"
#include "Engine/Scene.h"
#include "LocalLightSceneProxy.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalLightComponent)

ULocalLightComponent::ULocalLightComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Intensity = 5000;
	IntensityUnits = ELightUnits::Unitless;
	Radius_DEPRECATED = 1024.0f;
	AttenuationRadius = 1000;
	InverseExposureBlend = 0;
}

void ULocalLightComponent::SetAttenuationRadius(float NewRadius)
{
	// Only movable lights can change their radius at runtime
	if (AreDynamicDataChangesAllowed(false)
		&& NewRadius != AttenuationRadius)
	{
		AttenuationRadius = NewRadius;
		PushRadiusToRenderThread();
	}
}

void ULocalLightComponent::SetIntensityUnits(ELightUnits NewIntensityUnits)
{
	if (AreDynamicDataChangesAllowed()
		&& IntensityUnits != NewIntensityUnits)
	{
		IntensityUnits = NewIntensityUnits;

		UpdateColorAndBrightness();
	}
}

bool ULocalLightComponent::AffectsBounds(const FBoxSphereBounds& InBounds) const
{
	if((InBounds.Origin - GetComponentTransform().GetLocation()).SizeSquared() > FMath::Square(AttenuationRadius + InBounds.SphereRadius))
	{
		return false;
	}

	if(!Super::AffectsBounds(InBounds))
	{
		return false;
	}

	return true;
}

void ULocalLightComponent::SendRenderTransform_Concurrent()
{
	// Update the scene info's cached radius-dependent data.
	if(SceneProxy)
	{
		((FLocalLightSceneProxy*)SceneProxy)->UpdateRadius_GameThread(AttenuationRadius);
	}

	Super::SendRenderTransform_Concurrent();
}

FVector4 ULocalLightComponent::GetLightPosition() const
{
	return FVector4(GetComponentTransform().GetLocation(),1);
}

FBox ULocalLightComponent::GetBoundingBox() const
{
	return FBox(GetComponentLocation() - FVector(AttenuationRadius,AttenuationRadius,AttenuationRadius),GetComponentLocation() + FVector(AttenuationRadius,AttenuationRadius,AttenuationRadius));
}

FSphere ULocalLightComponent::GetBoundingSphere() const
{
	return FSphere(GetComponentTransform().GetLocation(), AttenuationRadius);
}

void ULocalLightComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.UEVer() < VER_UE4_INVERSE_SQUARED_LIGHTS_DEFAULT)
	{
		AttenuationRadius = Radius_DEPRECATED;
	}
}

#if WITH_EDITOR

bool ULocalLightComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bCastShadowsFromCinematicObjectsOnly) && bUseRayTracedDistanceFieldShadows)
		{
			return false;
		}
	}

	return Super::CanEditChange(InProperty);
}

/**
 * Called after property has changed via e.g. property window or set command.
 *
 * @param	PropertyThatChanged	FProperty that has been changed, NULL if unknown
 */
void ULocalLightComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Clamp intensity to 0 only for non-EV unit, as EV value are negative for small luminance value.
	if (IntensityUnits != ELightUnits::EV)
	{
		Intensity = FMath::Max(0.0f, Intensity);
	}
	LightmassSettings.IndirectLightingSaturation = FMath::Max(LightmassSettings.IndirectLightingSaturation, 0.0f);
	LightmassSettings.ShadowExponent = FMath::Clamp(LightmassSettings.ShadowExponent, .5f, 8.0f);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void ULocalLightComponent::PushRadiusToRenderThread()
{
	if (CastShadows)
	{
		// Shadow casting lights need to recompute light interactions
		// to determine which primitives to draw in shadow depth passes.
		MarkRenderStateDirty();
	}
	else
	{
		if (SceneProxy)
		{
			((FLocalLightSceneProxy*)SceneProxy)->UpdateRadius_GameThread(AttenuationRadius);
		}
	}
}

float ULocalLightComponent::GetUnitsConversionFactor(ELightUnits SrcUnits, ELightUnits TargetUnits, float CosHalfConeAngle)
{
	// Notes
	// -----
	// * UE light operates at constant 'luminous intensity' i.e., the intensity will remain constant when changing the light's 
	//   size (radius/width/height/...). When dealing with EV, we use an implicit 1m2 surface are for conversion, which allows 
	//   to keep EV constant under light's size change.
	// * UE unit is in centimeters (CM), while SI unit version are in meter (M), hence the conversion unit (100*100) in the 
	//   formula below
	// * When chaning unit, first GetUnitsConversionFactor() is called, then SetBrightness(). When switching to EV unit, 
	//   we for convert the intensity to luminance (assuming an implicity 1m2 surface) and then apply the luminance -> EV unit 
	//   in SetBrightness()
	// Reminder
	// --------
	// Light units (in terms of candela)
	//  Flux        = Lm = cd.sr
	//  Intensity   = Cd
	//  Luminance   = Cd/m2
	//  Illuminance = Cd.sr/m2 = Lux
	//
	// Light units (in terms of Lumen)
	//  Flux        = Lm = cd.sr
	//  Intensity   = Lm/sr
	//  Luminance   = Lm/sr/m2
	//  Illuminance = Lm/m2 = Lux
	
	CosHalfConeAngle = FMath::Clamp<float>(CosHalfConeAngle, -1, 1 - UE_KINDA_SMALL_NUMBER);

	if (SrcUnits == TargetUnits)
	{
		return 1.f;
	}
	else
	{
		float CnvFactor = 1.f;
		
		if (SrcUnits == ELightUnits::Nits)
		{
			// no scale factor (it depends on the light's area)
		}
		else if (SrcUnits == ELightUnits::Candelas)
		{
			CnvFactor = 100.f * 100.f;
		}
		else if (SrcUnits == ELightUnits::Lumens)
		{
			CnvFactor = 100.f * 100.f / 2.f / UE_PI / (1.f - CosHalfConeAngle);
		}
		else if (SrcUnits == ELightUnits::EV)
		{
			CnvFactor = 100.f * 100.f;
		}
		else
		{
			CnvFactor = 16.f;
		}

		if (TargetUnits == ELightUnits::Nits)
		{
			// no scale factor (it depends on the light's area)
		}
		else if (TargetUnits == ELightUnits::Candelas)
		{
			CnvFactor *= 1.f / 100.f / 100.f;
		}
		else if (TargetUnits == ELightUnits::Lumens)
		{
			CnvFactor *= 2.f  * UE_PI * (1.f - CosHalfConeAngle) / 100.f / 100.f;
		}
		else if (TargetUnits == ELightUnits::EV)
		{
			CnvFactor *= 1.f / 100.f / 100.f;
		}
		else
		{
			CnvFactor *= 1.f / 16.f;
		}

		return CnvFactor;
	}
}

void ULocalLightComponent::BuildSceneProxyDesc(const ULocalLightComponent& InLightComponent, FLocalLightSceneProxyDesc& OutProxyDesc)
{
	ULightComponent::BuildSceneProxyDesc(InLightComponent, OutProxyDesc);

	OutProxyDesc.MaxDrawDistance = InLightComponent.MaxDrawDistance;
	OutProxyDesc.MaxDistanceFadeRange = InLightComponent.MaxDistanceFadeRange;
	OutProxyDesc.InverseExposureBlend = InLightComponent.InverseExposureBlend;
	OutProxyDesc.AttenuationRadius = InLightComponent.AttenuationRadius;
}