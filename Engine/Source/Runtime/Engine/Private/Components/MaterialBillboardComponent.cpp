// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialBillboardComponent.h"
#include "PrimitiveViewRelevance.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/CollisionProfile.h"
#include "Curves/CurveFloat.h"
#include "SceneInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/Engine.h"
#include "PrimitiveSceneProxy.h"
#include "SceneView.h"
#include "StaticMeshResources.h"
#include "PSOPrecache.h"
#include "DataDrivenShaderPlatformInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialBillboardComponent)

/** A material sprite vertex. */
struct FMaterialSpriteVertex
{
	FVector3f Position;
	FPackedNormal TangentX;
	FPackedNormal TangentZ;
	FColor Color;
	FVector2f TexCoords;
};

/** A dummy vertex buffer used to give the FMaterialSpriteVertexFactory something to reference as a stream source. */
class FMaterialSpriteVertexBuffer : public FVertexBuffer
{
public:

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("FMaterialSpriteVertexBuffer"), sizeof(FMaterialSpriteVertex))
			.AddUsage(EBufferUsageFlags::Static)
			.DetermineInitialState();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}
};
static TGlobalResource<FMaterialSpriteVertexBuffer> GDummyMaterialSpriteVertexBuffer;

class FMaterialSpriteVertexArray : public FOneFrameResource
{
public:
	TArray<FMaterialSpriteVertex, TInlineAllocator<4> > Vertices;
};

/** Represents a sprite to the scene manager. */
class FMaterialSpriteSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	/** Initialization constructor. */
	FMaterialSpriteSceneProxy(const UMaterialBillboardComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
	, Elements(InComponent->Elements)
	, BaseColor(FColor::White)
	, VertexFactory(GetScene().GetFeatureLevel(), "FMaterialSpriteSceneProxy")
	{
		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
		{
			UMaterialInterface* Material = Elements[ElementIndex].Material;
			if (Material)
			{
				MaterialRelevance |= Material->GetRelevance_Concurrent(GetScene().GetShaderPlatform());
			}
		}

		// GetDynamicMeshElements needs to run single pass, because a single set of StaticMeshVertexBuffers are shared across all views.
		bSinglePassGDME = true;

		StaticMeshVertexBuffers.PositionVertexBuffer.Init(1);
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.Init(1, 1);
		StaticMeshVertexBuffers.ColorVertexBuffer.Init(1);

		const FMaterialSpriteSceneProxy* Self = this;
		ENQUEUE_RENDER_COMMAND(FMaterialSpriteSceneProxyInit)(
			[Self](FRHICommandListImmediate& RHICmdList)
		{
			Self->StaticMeshVertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
			Self->StaticMeshVertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
			Self->StaticMeshVertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			Self->StaticMeshVertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&Self->VertexFactory, Data);
			Self->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&Self->VertexFactory, Data);
			Self->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&Self->VertexFactory, Data);
			Self->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(&Self->VertexFactory, Data, 0);
			Self->StaticMeshVertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&Self->VertexFactory, Data);
			Self->VertexFactory.SetData(RHICmdList, Data);

			Self->VertexFactory.InitResource(RHICmdList);
		});
	}

	~FMaterialSpriteSceneProxy()
	{
		VertexFactory.ReleaseResource();
		StaticMeshVertexBuffers.PositionVertexBuffer.ReleaseResource();
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		StaticMeshVertexBuffers.ColorVertexBuffer.ReleaseResource();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER( STAT_MaterialSpriteSceneProxy_GetDynamicMeshElements );

		const int ElementStride = 4 * Elements.Num();
		const int WorstCaseVertexBufferSize = ElementStride * Views.Num();

		if (WorstCaseVertexBufferSize > 0)
		{
			StaticMeshVertexBuffers.PositionVertexBuffer.Init(WorstCaseVertexBufferSize);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.Init(WorstCaseVertexBufferSize, 1);
			StaticMeshVertexBuffers.ColorVertexBuffer.Init(WorstCaseVertexBufferSize);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];

					const bool bIsWireframe = View->Family->EngineShowFlags.Wireframe;
					// Determine the position of the source
					const FVector SourcePosition = GetLocalToWorld().GetOrigin();
					const FVector CameraToSource = View->ViewMatrices.GetViewOrigin() - SourcePosition;
					const float DistanceToSource = CameraToSource.Size();

					const FVector CameraUp = -View->ViewMatrices.GetClipToWorld().TransformVector(FVector(1.0f, 0.0f, 0.0f));
					const FVector CameraRight = -View->ViewMatrices.GetClipToWorld().TransformVector(FVector(0.0f, 1.0f, 0.0f));
					const FMatrix WorldToLocal = GetLocalToWorld().InverseFast();
					const FMatrix ViewToLocal = View->ViewMatrices.GetViewToWorld() * WorldToLocal;
					const FVector TangentX = -ViewToLocal.TransformVector(FVector(1.0f, 0.0f, 0.0f));
					const FVector TangentY = -ViewToLocal.TransformVector(FVector(0.0f, 1.0f, 0.0f));
					const FVector TangentZ = -ViewToLocal.TransformVector(FVector(0.0f, 0.0f, 1.0f));
					//				

					// Draw the elements ordered so the last is on top of the first.
					for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
					{
						const FMaterialSpriteElement& Element = Elements[ElementIndex];

						if (Element.Material)
						{
							// Evaluate the size of the sprite.
							float SizeX = Element.BaseSizeX;
							float SizeY = Element.BaseSizeY;
							if (Element.DistanceToSizeCurve)
							{
								const float SizeFactor = Element.DistanceToSizeCurve->GetFloatValue(DistanceToSource);
								SizeX *= SizeFactor;
								SizeY *= SizeFactor;
							}

							// Convert the size into world-space.
							const float W = View->ViewMatrices.GetWorldToClip().TransformPosition(SourcePosition).W;
							const float AspectRatio = CameraRight.Size() / CameraUp.Size();

							// TangentXYZ are normalized, so we don't / Right.Size() and Up.Size() again here
							const float WorldSizeX = Element.bSizeIsInScreenSpace ? (SizeX               * W) : SizeX;
							const float WorldSizeY = Element.bSizeIsInScreenSpace ? (SizeY * AspectRatio * W) : SizeY;

							// Evaluate the color/opacity of the sprite.
							FLinearColor Color = BaseColor;
							if (Element.DistanceToOpacityCurve)
							{
								Color.A *= Element.DistanceToOpacityCurve->GetFloatValue(DistanceToSource);
							}

							const int WriteOffset = ElementStride * ViewIndex + 4 * ElementIndex;
							for (uint32 VertexIndex = 0; VertexIndex < 4; ++VertexIndex)
							{
								const int WriteIndex = WriteOffset + VertexIndex;
								// correct TBN of billboard by ViewToLocal, notice that we use -TangentX
								StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(WriteIndex, (FVector3f)-TangentX, (FVector3f)TangentY, (FVector3f)TangentZ);
								StaticMeshVertexBuffers.ColorVertexBuffer.VertexColor(WriteIndex) = Color.ToFColor(true);
							}

							// Set up the sprite vertex positions and texture coordinates.
							StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(WriteOffset + 0) = FVector3f(-WorldSizeX * TangentY + +WorldSizeY * TangentX);
							StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(WriteOffset + 1) = FVector3f(+WorldSizeX * TangentY + +WorldSizeY * TangentX);
							StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(WriteOffset + 2) = FVector3f(-WorldSizeX * TangentY + -WorldSizeY * TangentX);
							StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(WriteOffset + 3) = FVector3f(+WorldSizeX * TangentY + -WorldSizeY * TangentX);

							StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(WriteOffset + 0, 0, FVector2f(0, 0));
							StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(WriteOffset + 1, 0, FVector2f(0, 1));
							StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(WriteOffset + 2, 0, FVector2f(1, 0));
							StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(WriteOffset + 3, 0, FVector2f(1, 1));

							// Set up the FMeshElement.
							FMeshBatch& Mesh = Collector.AllocateMesh();

							Mesh.VertexFactory = &VertexFactory;
							Mesh.MaterialRenderProxy = Element.Material->GetRenderProxy();
							Mesh.LCI = NULL;
							Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative() ? true : false;
							Mesh.CastShadow = false;
							Mesh.DepthPriorityGroup = (ESceneDepthPriorityGroup)GetDepthPriorityGroup(View);
							Mesh.Type = PT_TriangleStrip;
							Mesh.bDisableBackfaceCulling = true;

							// Set up the FMeshBatchElement.
							FMeshBatchElement& BatchElement = Mesh.Elements[0];
							BatchElement.IndexBuffer = NULL;
							BatchElement.FirstIndex = 0;
							BatchElement.MinVertexIndex = 0;
							BatchElement.MaxVertexIndex = 3;
							BatchElement.NumPrimitives = 2;
							BatchElement.BaseVertexIndex = WriteOffset;

							Mesh.bCanApplyViewModeOverrides = true;
							Mesh.bUseWireframeSelectionColoring = IsSelected();

							Collector.AddMesh(ViewIndex, Mesh);
						}
					}
				}
			}

			FLocalVertexFactory* VertexFactoryPtr = &VertexFactory;
			FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

			StaticMeshVertexBuffers.PositionVertexBuffer.UpdateRHI(RHICmdList);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.UpdateRHI(RHICmdList);
			StaticMeshVertexBuffers.ColorVertexBuffer.UpdateRHI(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			StaticMeshVertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactoryPtr, Data, 0);
			StaticMeshVertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(VertexFactoryPtr, Data);
			VertexFactoryPtr->SetData(RHICmdList, Data);
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		bool bVisible = View->Family->EngineShowFlags.BillboardSprites;
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = IsShadowCast(View);
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}
	virtual bool CanBeOccluded() const override { return !MaterialRelevance.bDisableDepthTest; }
	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }
	uint32 GetAllocatedSize() const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

private:
	/** The buffer containing vertex data. */
	mutable FStaticMeshVertexBuffers StaticMeshVertexBuffers;
	
	TArray<FMaterialSpriteElement> Elements;
	FMaterialRelevance MaterialRelevance;
	FColor BaseColor;
	mutable FLocalVertexFactory VertexFactory;
};

UMaterialBillboardComponent::UMaterialBillboardComponent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

FPrimitiveSceneProxy* UMaterialBillboardComponent::CreateSceneProxy()
{
	return new FMaterialSpriteSceneProxy(this);
}

FBoxSphereBounds UMaterialBillboardComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FTransform::FReal BoundsSize = 1.0f;
	for (int32 i = 0; i < Elements.Num(); ++i)
	{
		if (Elements[i].bSizeIsInScreenSpace)
		{
			// Workaround static bounds by disabling culling. Still allows override such as 'use parent bounds', etc.
			// Note: Bounds are dynamically calculated at draw time per view, so difficult to cull correctly. (UE-4725)
			BoundsSize = HALF_WORLD_MAX; 
			break;
		}
		else
		{
			BoundsSize = FMath::Max3<FTransform::FReal>(BoundsSize, Elements[i].BaseSizeX, Elements[i].BaseSizeY);
		}
	}
	BoundsSize *= LocalToWorld.GetMaximumAxisScale();

	return FBoxSphereBounds(LocalToWorld.GetLocation(),FVector(BoundsSize,BoundsSize,BoundsSize),FMath::Sqrt(3.0f * FMath::Square(BoundsSize)));
}

#if WITH_EDITOR
bool UMaterialBillboardComponent::GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty)
{
	if (Elements.IsValidIndex(ElementIndex))
	{
		OutOwner = this;
		OutPropertyPath = FString::Printf(TEXT("%s[%d].%s"), GET_MEMBER_NAME_STRING_CHECKED(UMaterialBillboardComponent, Elements), ElementIndex, GET_MEMBER_NAME_STRING_CHECKED(FMaterialSpriteElement, Material));
		OutProperty = FMaterialSpriteElement::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMaterialSpriteElement, Material));

		return true;
	}

	return false;
}
#endif // WITH_EDITOR

void UMaterialBillboardComponent::AddElement(
	class UMaterialInterface* Material,
	UCurveFloat* DistanceToOpacityCurve,
	bool bSizeIsInScreenSpace,
	float BaseSizeX,
	float BaseSizeY,
	UCurveFloat* DistanceToSizeCurve
	)
{
	FMaterialSpriteElement* Element = new(Elements) FMaterialSpriteElement;
	Element->Material = Material;
	Element->DistanceToOpacityCurve = DistanceToOpacityCurve;
	Element->bSizeIsInScreenSpace = bSizeIsInScreenSpace;
	Element->BaseSizeX = BaseSizeX;
	Element->BaseSizeY = BaseSizeY;
	Element->DistanceToSizeCurve = DistanceToSizeCurve;

	PrecachePSOs();

	MarkRenderStateDirty();
}

void UMaterialBillboardComponent::SetElements(const TArray<FMaterialSpriteElement>& NewElements)
{
	// Replace existing array
	Elements = NewElements;

	PrecachePSOs();

	// Indicate scene proxy needs to be updated
	MarkRenderStateDirty();
}


UMaterialInterface* UMaterialBillboardComponent::GetMaterial(int32 Index) const
{
	UMaterialInterface* ResultMI = nullptr;
	if (Elements.IsValidIndex(Index))
	{
		ResultMI = Elements[Index].Material;
	}
	return ResultMI;
}

void UMaterialBillboardComponent::PostLoad()
{
	Super::PostLoad();

	PrecachePSOs();
}

void UMaterialBillboardComponent::SetMaterial(int32 ElementIndex, class UMaterialInterface* Material)
{
	if (Elements.IsValidIndex(ElementIndex))
	{
		Elements[ElementIndex].Material = Material;

		PrecachePSOs();

		MarkRenderStateDirty();
	}
}

void UMaterialBillboardComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		OutMaterials.AddUnique(GetMaterial(ElementIndex));
	}
}

void UMaterialBillboardComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	FPSOPrecacheParams PrecachePSOParams = BasePrecachePSOParams;
	PrecachePSOParams.PrimitiveType = PT_TriangleStrip;
	PrecachePSOParams.bDisableBackFaceCulling = true;
 
	const FVertexFactoryType* VFType = &FLocalVertexFactory::StaticType;
 
	TArray<UMaterialInterface*> UsedMaterials;
	GetUsedMaterials(UsedMaterials, false);
	for (UMaterialInterface* MaterialInterface : UsedMaterials)
	{
		if (MaterialInterface)
		{
			FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
			ComponentParams.MaterialInterface = MaterialInterface;
			ComponentParams.PSOPrecacheParams = PrecachePSOParams;

			if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
			{
				ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(VFType));
			}
			else
			{
				FVertexDeclarationElementList VtxElements;
				VtxElements.Add(FVertexElement(0, 0, VET_Float3, 0, sizeof(FVector3f)));
				VtxElements.Add(FVertexElement(1, 0, VET_PackedNormal, 1, sizeof(FPackedNormal) * 2));
				VtxElements.Add(FVertexElement(1, sizeof(FPackedNormal), VET_PackedNormal, 2, sizeof(FPackedNormal) * 2));
				VtxElements.Add(FVertexElement(2, 0, VET_Color, 3, sizeof(FColor)));
				VtxElements.Add(FVertexElement(3, 0, VET_Half2, 4, 4));
				VtxElements.Add(FVertexElement(3, 0, VET_Half2, 5, 4));
				VtxElements.Add(FVertexElement(3, 0, VET_Half2, 6, 4));
				VtxElements.Add(FVertexElement(3, 0, VET_Half2, 7, 4));
				VtxElements.Add(FVertexElement(3, 0, VET_Half2, 15, 4));
				if (IsGPUSkinPassThroughSupported(GMaxRHIShaderPlatform))
				{
					VtxElements.Add(FVertexElement(4, 0, VET_Float4, 14, 0));
					VtxElements.Add(FVertexElement(5, 0, VET_UInt, 13, 0, true));
				}
				else
				{
					VtxElements.Add(FVertexElement(4, 0, VET_UInt, 13, 0, true));
				}
				ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(VFType, VtxElements));
			}
		}
	}
}

