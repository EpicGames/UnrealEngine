// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MeshComponent.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "Engine/MaterialOverlayHelper.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ContentStreaming.h"
#include "Materials/MaterialRelevance.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Engine/World.h"
#include "PSOPrecache.h"
#include "UObject/UnrealType.h"
#include "StaticMeshSceneProxyDesc.h"
#include "MeshComponentHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshComponent)

#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#include "TextureCompiler.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogMaterialParameter, Warning, All);

UMeshComponent::UMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CastShadow = true;
	bUseAsOccluder = true;
	bCanEverAffectNavigation = true;
	bCachedMaterialParameterIndicesAreDirty = true;
	bEnableMaterialParameterCaching = false;
}

UMaterialInterface* UMeshComponent::GetMaterial(int32 ElementIndex) const
{
	UMaterialInterface* OutMaterial = nullptr;

	if (OverrideMaterials.IsValidIndex(ElementIndex))
	{
		OutMaterial = OverrideMaterials[ElementIndex];
	}

	if (OutMaterial != nullptr && UseNaniteOverrideMaterials())
	{
		UMaterialInterface* NaniteOverride = OutMaterial->GetNaniteOverride();
		OutMaterial = NaniteOverride != nullptr ? NaniteOverride : OutMaterial;
	}

	return OutMaterial;
}

UMaterialInterface* UMeshComponent::GetMaterialByName(FName MaterialSlotName) const
{
	int32 MaterialIndex = GetMaterialIndex(MaterialSlotName);
	if (MaterialIndex < 0)
		return nullptr;
	return GetMaterial(MaterialIndex);
}

void UMeshComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* Material)
{
	if (ElementIndex >= 0)
	{
		if (OverrideMaterials.IsValidIndex(ElementIndex) && (OverrideMaterials[ElementIndex] == Material))
		{
			// Do nothing, the material is already set
		}
		else
		{
			// Grow the array if the new index is too large
			if (OverrideMaterials.Num() <= ElementIndex)
			{
				OverrideMaterials.AddZeroed(ElementIndex + 1 - OverrideMaterials.Num());
			}
			
			// Check if we are setting a dynamic instance of the original material, or replacing a nullptr material  (if not we should dirty the material parameter name cache)
			if (Material != nullptr)
			{
				UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(Material);
				if (!((DynamicMaterial != nullptr && DynamicMaterial->Parent == OverrideMaterials[ElementIndex]) || OverrideMaterials[ElementIndex] == nullptr))
				{
					// Mark cached material parameter names dirty
					MarkCachedMaterialParameterNameIndicesDirty();
				}
			}	

			if (HasBegunPlay())
			{
				if (UMaterialInterface* PreviousMaterial = OverrideMaterials[ElementIndex].Get())
				{
					PreviousMaterial->OnRemovedAsOverride(GetOwner());
				}
			}

			// Set the material and invalidate things
			OverrideMaterials[ElementIndex] = Material;

			if (HasBegunPlay())
			{
				if (Material)
				{
					Material->OnAssignedAsOverride(GetOwner());
				}
			}

			// Precache PSOs again
			PrecachePSOs();

			MarkRenderStateDirty();
			// If MarkRenderStateDirty didn't notify the streamer, do it now
			if (!IsIgnoreStreamingManagerUpdate() && OwnerLevelHasRegisteredStaticComponentsInStreamingManager(GetOwner()))
			{
				IStreamingManager::Get().NotifyPrimitiveUpdated_Concurrent(this);
			}
			if (Material)
			{
				Material->AddToCluster(this, true);
			}

			FBodyInstance* BodyInst = GetBodyInstance();
			if (BodyInst && BodyInst->IsValidBodyInstance())
			{
				BodyInst->UpdatePhysicalMaterials();
			}

#if WITH_EDITOR
			// Static Lighting is updated when compilation finishes
			if (!IsCompiling())
			{
				FStaticLightingSystemInterface::OnPrimitiveComponentUnregistered.Broadcast(this);
				if (HasValidSettingsForStaticLighting(false))
				{
					FStaticLightingSystemInterface::OnPrimitiveComponentRegistered.Broadcast(this);
				}
			}
#endif
		}
	}
}

void UMeshComponent::SetMaterialByName(FName MaterialSlotName, UMaterialInterface* Material)
{
	int32 MaterialIndex = GetMaterialIndex(MaterialSlotName);
	if (MaterialIndex < 0)
		return;

	SetMaterial(MaterialIndex, Material);
}

// Deprecated in 5.7
FMaterialRelevance UMeshComponent::GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UMeshComponent::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	return FMeshComponentHelper::GetMaterialRelevance(*this, InShaderPlatform);
}

int32 UMeshComponent::GetNumOverrideMaterials() const
{
	return OverrideMaterials.Num();
}

#if WITH_EDITOR
void UMeshComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(UMeshComponent, OverrideMaterials))
		{
			CleanUpOverrideMaterials();
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(UMeshComponent, MaterialSlotsOverlayMaterial))
		{
			CleanUpMaterialSlotsOverlayMaterial();
		}
	}
}

void UMeshComponent::CleanUpOverrideMaterials()
{
	bool bUpdated = false;
	int32 NumMaterials = GetNumMaterials();
	int32 NumOverrideMaterials = OverrideMaterials.Num();

	// We have to remove material override Ids that are bigger then the material list
	if (NumOverrideMaterials > NumMaterials)
	{
		//Remove the override material id that are superior to the static mesh materials number
		int32 RemoveCount = NumOverrideMaterials - NumMaterials;

		if (HasBegunPlay())
		{
			for (int32 MatIndex = NumMaterials; MatIndex < NumOverrideMaterials; MatIndex++)
			{
				if (UMaterialInterface* MatInterface = OverrideMaterials[MatIndex].Get())
				{
					MatInterface->OnRemovedAsOverride(GetOwner());
				}
			}
		}

		OverrideMaterials.RemoveAt(NumMaterials, RemoveCount);
		bUpdated = true;
	}

	if (bUpdated)
	{
		MarkRenderStateDirty();
	}
}

void UMeshComponent::CleanUpMaterialSlotsOverlayMaterial()
{
	//Calling the default will fill all the material slot until we have 
	const int32 AssetMaterialSlotsOverlayMaterialCount = GetNumMaterials();
	const int32 ComponentMaterialSlotsOverlayMaterialCount = MaterialSlotsOverlayMaterial.Num();
	if (ComponentMaterialSlotsOverlayMaterialCount == 0)
	{
		return;
	}

	bool bUpdated = false;
	if (ComponentMaterialSlotsOverlayMaterialCount > AssetMaterialSlotsOverlayMaterialCount)
	{
		int32 RemoveCount = ComponentMaterialSlotsOverlayMaterialCount - AssetMaterialSlotsOverlayMaterialCount;
		MaterialSlotsOverlayMaterial.RemoveAt(AssetMaterialSlotsOverlayMaterialCount, RemoveCount);
		bUpdated = true;
	}

	if (bUpdated)
	{
		MarkRenderStateDirty();
	}
}

#endif

void UMeshComponent::EmptyOverrideMaterials()
{
	bool bRefresh = false;
	if (OverrideMaterials.Num())
	{
		if (HasBegunPlay())
		{
			for (int32 MatIndex = 0; MatIndex < OverrideMaterials.Num(); MatIndex++)
			{
				if (UMaterialInterface* MatInterface = OverrideMaterials[MatIndex].Get())
				{
					MatInterface->OnRemovedAsOverride(GetOwner());
				}
			}
		}

		OverrideMaterials.Reset();
		bRefresh = true;
	}

	//Empty the material slot overlay material array
	//Those are hook to the material list like the override materials
	if (MaterialSlotsOverlayMaterial.Num())
	{
		MaterialSlotsOverlayMaterial.Reset();
		bRefresh = true;
	}

	if(bRefresh)
	{
		MarkRenderStateDirty();

		// Precache PSOs again
		PrecachePSOs();
	}
}

bool UMeshComponent::HasOverrideMaterials()
{
	return OverrideMaterials.Num() > 0;
}

void UMeshComponent::BeginPlay()
{
	Super::BeginPlay();

	for (int32 MatIndex = 0; MatIndex < OverrideMaterials.Num(); MatIndex++)
	{
		if (UMaterialInterface* MatInterface = OverrideMaterials[MatIndex].Get())
		{
			MatInterface->OnAssignedAsOverride(GetOwner());
		}
	}
}

void UMeshComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	for (int32 MatIndex = 0; MatIndex < OverrideMaterials.Num(); MatIndex++)
	{
		if (UMaterialInterface* MatInterface = OverrideMaterials[MatIndex].Get())
		{
			MatInterface->OnRemovedAsOverride(GetOwner());
		}
	}

	Super::EndPlay(EndPlayReason);
}

int32 UMeshComponent::GetNumMaterials() const
{
	return 0;
}

void UMeshComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	for (int32 ElementIndex = 0; ElementIndex < GetNumMaterials(); ElementIndex++)
	{
		if (UMaterialInterface* MaterialInterface = GetMaterial(ElementIndex))
		{
			OutMaterials.Add(MaterialInterface);
		}
	}

	//Add material slot overlay materials
	TArray<TObjectPtr<UMaterialInterface>> AssetAndComponentMaterialSlotOverlayMaterials;
	GetMaterialSlotsOverlayMaterial(AssetAndComponentMaterialSlotOverlayMaterials);
	FMaterialOverlayHelper::AppendAllOverlayMaterial(AssetAndComponentMaterialSlotOverlayMaterials, OutMaterials);

	//Add global mesh overlay
	UMaterialInterface* OverlayMaterialInterface = GetOverlayMaterial();
	if (OverlayMaterialInterface != nullptr)
	{
		OutMaterials.Add(OverlayMaterialInterface);
	}
}

UMaterialInterface* UMeshComponent::GetOverlayMaterial(bool bGetMaterialSlot, int32 SlotIndex) const
{
	if (bGetMaterialSlot)
	{
		if (MaterialSlotsOverlayMaterial.IsValidIndex(SlotIndex))
		{
			return MaterialSlotsOverlayMaterial[SlotIndex];
		}
	}
	else if (OverlayMaterial)
	{ 
		return OverlayMaterial;
	}

	return GetDefaultOverlayMaterial();
}

void UMeshComponent::SetOverlayMaterial(UMaterialInterface* NewOverlayMaterial, bool bSetMaterialSlot, int32 SlotIndex)
{
	TObjectPtr<UMaterialInterface>* OverrideOverlaySlot = nullptr;

	if (bSetMaterialSlot)
	{
		if (SlotIndex < 0)
		{
			return;
		}

		bool bValid = MaterialSlotsOverlayMaterial.IsValidIndex(SlotIndex);
		if (!bValid)
		{
			if (SlotIndex < GetNumMaterials())
			{
				MaterialSlotsOverlayMaterial.SetNum(SlotIndex + 1);
				bValid = true;
			}
		}

		if (bValid)
		{
			OverrideOverlaySlot = &MaterialSlotsOverlayMaterial[SlotIndex];
		}
	}
	else
	{
		OverrideOverlaySlot = &OverlayMaterial;
	}

	if (OverrideOverlaySlot && *OverrideOverlaySlot != NewOverlayMaterial)
	{
		*OverrideOverlaySlot = NewOverlayMaterial;
		// Precache PSOs again
		PrecachePSOs();
		MarkRenderStateDirty();
	}
}

float UMeshComponent::GetOverlayMaterialMaxDrawDistance() const
{
	if (OverlayMaterialMaxDrawDistance != 0.f)
	{ 
		return OverlayMaterialMaxDrawDistance;
	}
	else
	{
		return GetDefaultOverlayMaterialMaxDrawDistance();
	}
}

void UMeshComponent::SetOverlayMaterialMaxDrawDistance(float InMaxDrawDistance)
{
	if (OverlayMaterialMaxDrawDistance != InMaxDrawDistance)
	{
		OverlayMaterialMaxDrawDistance = InMaxDrawDistance;
		MarkRenderStateDirty();
	}
}

const TArray<TObjectPtr<UMaterialInterface>>& UMeshComponent::GetComponentMaterialSlotsOverlayMaterial() const
{
	return MaterialSlotsOverlayMaterial;
}

void UMeshComponent::GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const
{
	FMeshComponentHelper::GetMaterialSlotsOverlayMaterial(*this, OutMaterialSlotOverlayMaterials);
}

void UMeshComponent::PrestreamTextures( float Seconds, bool bPrioritizeCharacterTextures, int32 CinematicTextureGroups )
{
	// If requested, tell the streaming system to only process character textures for 30 frames.
	if (bPrioritizeCharacterTextures)
	{
		IStreamingManager::Get().SetDisregardWorldResourcesForFrames(30);
	}

	TArray<UTexture*> Textures;
	GetUsedTextures(/*out*/ Textures, GetCurrentMaterialQualityLevelChecked());

#if WITH_EDITOR
	FTextureCompilingManager::Get().FinishCompilation(Textures);
#endif

	for (UTexture* Texture : Textures)
	{
		if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
		{
			Texture2D->SetForceMipLevelsToBeResident(Seconds, CinematicTextureGroups);
		}
	}
}

void UMeshComponent::RegisterLODStreamingCallback(FLODStreamingCallback&& Callback, int32 LODIdx, float TimeoutSecs, bool bOnStreamIn)
{
	check(IsInGameThread());
	Callback(GetPrimitiveComponentInterface(), nullptr, ELODStreamingCallbackResult::NotImplemented);
}

void UMeshComponent::RegisterLODStreamingCallback(FLODStreamingCallback&& CallbackStreamingStart, FLODStreamingCallback&& CallbackStreamingDone, float TimeoutStartSecs, float TimeoutDoneSecs)
{
	check(IsInGameThread());
	CallbackStreamingDone(GetPrimitiveComponentInterface(), nullptr, ELODStreamingCallbackResult::NotImplemented);
}

void UMeshComponent::SetTextureForceResidentFlag( bool bForceMiplevelsToBeResident )
{
	const int32 CinematicTextureGroups = 0;
	const float Seconds = -1.0f;

	TArray<UTexture*> Textures;
	GetUsedTextures(/*out*/ Textures, GetCurrentMaterialQualityLevelChecked());

#if WITH_EDITOR
	FTextureCompilingManager::Get().FinishCompilation(Textures);
#endif

	for (UTexture* Texture : Textures)
	{
		if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
		{
			Texture2D->SetForceMipLevelsToBeResident(Seconds, CinematicTextureGroups);
			Texture2D->bForceMiplevelsToBeResident = bForceMiplevelsToBeResident;
		}
	}
}

#if WITH_EDITOR
void UMeshComponent::SetMarkTextureAsEditorStreamingPool(bool bInMarkAsEditorStreamingPool)
{
	TArray<UTexture*> Textures;
	GetUsedTextures(/*out*/ Textures, GetCurrentMaterialQualityLevelChecked());
	FTextureCompilingManager::Get().FinishCompilation(Textures);

	for (UTexture* Texture : Textures)
	{
		Texture->bMarkAsEditorStreamingPool = bInMarkAsEditorStreamingPool;
	}
}
#endif

TArray<class UMaterialInterface*> UMeshComponent::GetMaterials() const
{
	TArray<class UMaterialInterface*> OutMaterials;
	int32 TotalNumMaterials = GetNumMaterials();
	if(TotalNumMaterials > 0)
	{
		// make sure to extend it
		OutMaterials.AddZeroed(TotalNumMaterials);

		for(int32 MaterialIndex=0; MaterialIndex < TotalNumMaterials; ++MaterialIndex)
		{
			OutMaterials[MaterialIndex] = GetMaterial(MaterialIndex);
		}
	}

	return OutMaterials;
}

void UMeshComponent::SetParameterValueOnMaterials(const FName ParameterName
												, const TFunction<void(UMaterialInstanceDynamic* DynamicMaterial)>& SetParameterFunction
												, const TFunction<const TArray<int32>& (const FMaterialParameterCache* ParameterCache)>& GetMaterialIndicesFromCacheFunction)
{
	bool bDynamicSlotOverlayMaterialCreated = false;

	TArray<TObjectPtr<UMaterialInterface>> RetrievedMaterialSlotsOverlayMaterials;
	GetMaterialSlotsOverlayMaterial(RetrievedMaterialSlotsOverlayMaterials);
	MaterialSlotsOverlayMaterial.SetNum(RetrievedMaterialSlotsOverlayMaterials.Num());

	auto CreateMaterialInstanceDynamic = [this](UMaterialInterface* MaterialInterface, int32 MaterialIndex)
		{
			return CreateAndSetMaterialInstanceDynamic(MaterialIndex);
		};

	auto CreateMaterialSlotsOverlayMaterialInstanceDynamic = [this, &bDynamicSlotOverlayMaterialCreated](UMaterialInterface* MaterialInterface, int32 MaterialIndex)
		{
			bDynamicSlotOverlayMaterialCreated = true;

			UMaterialInstanceDynamic* DynamicMaterial = UMaterialInstanceDynamic::Create(MaterialInterface, this);
			MaterialSlotsOverlayMaterial[MaterialIndex] = DynamicMaterial;

			return DynamicMaterial;
		};

	if (!bEnableMaterialParameterCaching)
	{
		auto SetParameterValueOnMaterials_Internal = [&SetParameterFunction]
			(const TArray<UMaterialInterface*>& MaterialInterfaces
			, const TFunction<UMaterialInstanceDynamic*(UMaterialInterface* MaterialInterface, int32 MaterialIndex)>& CreateDynamicMaterialInstanceFunction
			)
			{
				for (int32 MaterialIndex = 0; MaterialIndex < MaterialInterfaces.Num(); ++MaterialIndex)
				{
					UMaterialInterface* MaterialInterface = MaterialInterfaces[MaterialIndex];
					if (MaterialInterface)
					{
						UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(MaterialInterface);
						if (!DynamicMaterial)
						{
							DynamicMaterial = CreateDynamicMaterialInstanceFunction(MaterialInterface, MaterialIndex);
						}
						SetParameterFunction(DynamicMaterial);
					}
				}
			};

		SetParameterValueOnMaterials_Internal(GetMaterials(), CreateMaterialInstanceDynamic);
		SetParameterValueOnMaterials_Internal(RetrievedMaterialSlotsOverlayMaterials, CreateMaterialSlotsOverlayMaterialInstanceDynamic);
	}
	else
	{
		if (bCachedMaterialParameterIndicesAreDirty)
		{
			CacheMaterialParameterNameIndices();
		}

		auto GetMaterialAtIndex = [this](int32 MaterialIndex) { return GetMaterial(MaterialIndex); };
		auto GetMaterialSlotOverlayMaterialAtIndex = [RetrievedMaterialSlotsOverlayMaterials](int32 MaterialIndex) { return RetrievedMaterialSlotsOverlayMaterials[MaterialIndex]; };

		auto SetParameterValueOnMaterials_Internal = [&SetParameterFunction, &GetMaterialIndicesFromCacheFunction, ParameterName, this]
			(FMaterialParameterCache* ParameterCache
			, const TFunction<UMaterialInterface*(int32 MaterialIndex)>& GetMaterialFunction
			, const TFunction<UMaterialInstanceDynamic* (UMaterialInterface* MaterialInterface, int32 MaterialIndex)>& CreateDynamicMaterialInstanceFunction
			)
			{
				if (ParameterCache != nullptr)
				{
					const TArray<int32>& MaterialIndices = GetMaterialIndicesFromCacheFunction(ParameterCache);
					// Loop over all the material indices and update set the parameter value on the corresponding materials		
					for (int32 MaterialIndex : MaterialIndices)
					{
						UMaterialInterface* MaterialInterface = GetMaterialFunction(MaterialIndex);
						if (MaterialInterface)
						{
							UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(MaterialInterface);
							if (!DynamicMaterial)
							{
								DynamicMaterial = CreateDynamicMaterialInstanceFunction(MaterialInterface, MaterialIndex);
							}
							SetParameterFunction(DynamicMaterial);
						}
					}
				}
				else
				{
					UE_LOGF(LogMaterialParameter, Log, "%ls material parameter hasn't found on the component %ls", *ParameterName.ToString(), *GetPathName());
				}
			};

		SetParameterValueOnMaterials_Internal(MaterialParameterCache.Find(ParameterName), GetMaterialAtIndex, CreateMaterialInstanceDynamic);
		SetParameterValueOnMaterials_Internal(MaterialSlotsOverlayMaterialParameterCache.Find(ParameterName), GetMaterialSlotOverlayMaterialAtIndex, CreateMaterialSlotsOverlayMaterialInstanceDynamic);
	}

	if (bDynamicSlotOverlayMaterialCreated)
	{
		PrecachePSOs();
		MarkRenderStateDirty();
	}
}

void UMeshComponent::SetScalarParameterValueOnMaterials(const FName ParameterName, const float ParameterValue)
{
	SetParameterValueOnMaterials(ParameterName
								, [ParameterName, ParameterValue](UMaterialInstanceDynamic* DynamicMaterial) { DynamicMaterial->SetScalarParameterValue(ParameterName, ParameterValue); }
								, [](const FMaterialParameterCache* ParameterCache) -> const TArray<int32>& { return ParameterCache->ScalarParameterMaterialIndices; });
}

void UMeshComponent::SetVectorParameterValueOnMaterials(const FName ParameterName, const FVector ParameterValue)
{
	SetColorParameterValueOnMaterials(ParameterName, FLinearColor(ParameterValue));
}

void UMeshComponent::SetColorParameterValueOnMaterials(const FName ParameterName, const FLinearColor ParameterValue)
{
	SetParameterValueOnMaterials(ParameterName
								, [ParameterName, ParameterValue](UMaterialInstanceDynamic* DynamicMaterial) { DynamicMaterial->SetVectorParameterValue(ParameterName, ParameterValue); }
								, [](const FMaterialParameterCache* ParameterCache) -> const TArray<int32>& { return ParameterCache->VectorParameterMaterialIndices; });
}

void UMeshComponent::MarkCachedMaterialParameterNameIndicesDirty()
{
	// Flag the cached material parameter indices as dirty
	bCachedMaterialParameterIndicesAreDirty = true;
}

void UMeshComponent::BeginDestroy()
{
	for (int32 MatIndex = 0; MatIndex < OverrideMaterials.Num(); MatIndex++)
	{
		OverrideMaterials[MatIndex] = nullptr;
	}

	Super::BeginDestroy();
}

void UMeshComponent::CacheMaterialParameterNameIndices()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CacheMaterialParameterNameIndices);
	if (!bEnableMaterialParameterCaching)
	{
		return;
	}

	// not sure if this is the best way to do this
	const UWorld* World = GetWorld();
	// to set the default value for scalar params, we use a FMaterialResource, which means the world has to be rendering
	const bool bHasMaterialResource = (World && World->WorldType != EWorldType::Inactive);

	auto CacheMaterialParameterNameIndices_Internal = [bHasMaterialResource](const TArray<UMaterialInterface*>& MaterialInterfacesToCache, TSortedMap<FName, FMaterialParameterCache, FDefaultAllocator, FNameFastLess>& MaterialParameterCacheToUpdate)
		{
			// Clean up possible previous data
			MaterialParameterCacheToUpdate.Reset();
			
			int32 MaterialIndex = 0;
			for (UMaterialInterface* MaterialInterface : MaterialInterfacesToCache)
			{
				if (MaterialInterface)
				{
					TArray<FMaterialParameterInfo> OutParameterInfo;
					TArray<FGuid> OutParameterIds;

					// Retrieve all scalar parameter names from the material
					MaterialInterface->GetAllScalarParameterInfo(OutParameterInfo, OutParameterIds);
					for (FMaterialParameterInfo& ParameterInfo : OutParameterInfo)
					{
						// Add or retrieve entry for this parameter name
						FMaterialParameterCache& ParameterCache = MaterialParameterCacheToUpdate.FindOrAdd(ParameterInfo.Name);
						// Add the corresponding material index
						ParameterCache.ScalarParameterMaterialIndices.Add(MaterialIndex);
						
						// GetScalarParameterDefault() expects to use a FMaterialResource, which means the world has to be rendering
						if (bHasMaterialResource)
						{
							// store the default value
							 MaterialInterface->GetScalarParameterDefaultValue(ParameterInfo, ParameterCache.ScalarParameterDefaultValue);
						}
					}

					// Empty parameter names and ids
					OutParameterInfo.Reset();
					OutParameterIds.Reset();

					// Retrieve all vector parameter names from the material
					MaterialInterface->GetAllVectorParameterInfo(OutParameterInfo, OutParameterIds);
					for (FMaterialParameterInfo& ParameterInfo : OutParameterInfo)
					{
						// Add or retrieve entry for this parameter name
						FMaterialParameterCache& ParameterCache = MaterialParameterCacheToUpdate.FindOrAdd(ParameterInfo.Name);
						// Add the corresponding material index
						ParameterCache.VectorParameterMaterialIndices.Add(MaterialIndex);
					}
				}
				++MaterialIndex;
			}
		};

	CacheMaterialParameterNameIndices_Internal(GetMaterials(), MaterialParameterCache);

	TArray<TObjectPtr<UMaterialInterface>> RetrievedMaterialSlotsOverlayMaterials;
	GetMaterialSlotsOverlayMaterial(RetrievedMaterialSlotsOverlayMaterials);
	CacheMaterialParameterNameIndices_Internal(RetrievedMaterialSlotsOverlayMaterials, MaterialSlotsOverlayMaterialParameterCache);

	bCachedMaterialParameterIndicesAreDirty = false;
}

void UMeshComponent::GetStreamingTextureInfoInner(FStreamingTextureLevelContext& LevelContext, const TArray<FStreamingTextureBuildInfo>* PreBuiltData, float ComponentScaling, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingTextures) const
{
	if (CanSkipGetTextureStreamingRenderAssetInfo())
	{
		return;
	}

	LevelContext.BindBuildData(PreBuiltData);

	const int32 NumMaterials = GetNumMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
	{
		FPrimitiveMaterialInfo MaterialData;
		if (GetMaterialStreamingData(MaterialIndex, MaterialData))
		{
			LevelContext.ProcessMaterial(Bounds, MaterialData, ComponentScaling, OutStreamingTextures, bIsValidTextureStreamingBuiltData);
		}
	}
}

FColor UMeshComponent::GetWireframeColorForSceneProxy() const
{
	if (Mobility == EComponentMobility::Static)
	{
		return FColor(0, 255, 255, 255);
	}
	else if (Mobility == EComponentMobility::Stationary)
	{
		return FColor(128, 128, 255, 255);
	}
	else // Movable
	{
		if (BodyInstance.bSimulatePhysics)
		{
			return FColor(0, 255, 128, 255);
		}
		else
		{
			return FColor(255, 0, 255, 255);
		}
	}
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

void UMeshComponent::LogMaterialsAndTextures(FOutputDevice& Ar, int32 Indent) const
{
	Ar.Logf(TEXT("%s%s:"), FCString::Tab(Indent), *GetClass()->GetName());

	for (int32 MaterialIndex = 0; MaterialIndex < OverrideMaterials.Num(); ++MaterialIndex)
	{
		Ar.Logf(TEXT("%s[Material Override: %d]"), FCString::Tab(Indent + 1), MaterialIndex);
		const UMaterialInterface* MaterialInterface = OverrideMaterials[MaterialIndex];
		if (MaterialInterface)
		{
			MaterialInterface->LogMaterialsAndTextures(Ar, Indent + 2);
		}
		else
		{
			Ar.Logf(TEXT("%snullptr"), FCString::Tab(Indent + 2), MaterialIndex);
		}
	}

	// Backup the material overrides so we can access the mesh original materials.
	TArray<TObjectPtr<class UMaterialInterface>> OverrideMaterialsBackup;
	Swap(OverrideMaterialsBackup, const_cast<UMeshComponent*>(this)->OverrideMaterials);

	TArray<UMaterialInterface*> MaterialInterfaces = GetMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < MaterialInterfaces.Num(); ++MaterialIndex)
	{
		Ar.Logf(TEXT("%s[Mesh Material: %d]"), FCString::Tab(Indent + 1), MaterialIndex);
		const UMaterialInterface* MaterialInterface = MaterialInterfaces[MaterialIndex];
		if (MaterialInterface)
		{
			MaterialInterface->LogMaterialsAndTextures(Ar, Indent + 2);
		}
		else
		{
			Ar.Logf(TEXT("%snullptr"), FCString::Tab(Indent + 2), MaterialIndex);
		}
	}

	// Restore the overrides.
	Swap(OverrideMaterialsBackup, const_cast<UMeshComponent*>(this)->OverrideMaterials);
}

#endif

