// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ModelComponent.cpp: Model component implementation
=============================================================================*/

#include "Components/ModelComponent.h"
#include "BodySetupEnums.h"
#include "Engine/Level.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "EngineLogs.h"
#include "MaterialDomain.h"
#include "Engine/CollisionProfile.h"
#include "LightMap.h"
#include "Model.h"
#include "ShadowMap.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/BodySetup.h"
#include "UObject/UObjectAnnotation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelComponent)

FModelElement::FModelElement(UModelComponent* InComponent,UMaterialInterface* InMaterial):
	Component(InComponent),
	Material(InMaterial),
	LegacyMapBuildData(NULL),
	IndexBuffer(NULL),
	FirstIndex(0),
	NumTriangles(0),
	MinVertexIndex(0),
	MaxVertexIndex(0),
	BoundingBox(ForceInitToZero)
{
	MapBuildDataId = FGuid::NewGuid();
}

FModelElement::FModelElement():
	Component(nullptr), 
	Material(nullptr), 
	LegacyMapBuildData(NULL),
	IndexBuffer(NULL), 
	FirstIndex(0), 
	NumTriangles(0), 
	MinVertexIndex(0), 
	MaxVertexIndex(0), 
	BoundingBox(ForceInitToZero)
{
}

FModelElement::~FModelElement()
{}

const FMeshMapBuildData* FModelElement::GetMeshMapBuildData() const
{
	check(Component);
	ULevel* OwnerLevel = Cast<ULevel>(Component->GetModel()->GetOuter());

	if (OwnerLevel && OwnerLevel->OwningWorld)
	{
		if (UMapBuildDataRegistry* MapBuildData = UMapBuildDataRegistry::Get(OwnerLevel, OwnerLevel->OwningWorld))
		{
			return MapBuildData->GetMeshBuildData(MapBuildDataId);		
		}
	}
	
	return NULL;
}

/**
 * Serializer.
 */
FArchive& operator<<(FArchive& Ar,FModelElement& Element)
{
	if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MapBuildDataSeparatePackage)
	{
		Element.LegacyMapBuildData = new FMeshMapBuildData();
		Ar << Element.LegacyMapBuildData->LightMap;
		Ar << Element.LegacyMapBuildData->ShadowMap;
	}

	if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::FixedBSPLightmaps)
	{
		Ar << Element.MapBuildDataId;
	}
	else if (Ar.IsLoading())
	{
		Element.MapBuildDataId = FGuid::NewGuid();
	}

	Ar << (UObject*&)Element.Component << (UObject*&)Element.Material << Element.Nodes;

	if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MapBuildDataSeparatePackage)
	{
		Ar << Element.LegacyMapBuildData->IrrelevantLights;
	}

	return Ar;
}

UModelComponent::UModelComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CastShadow = true;
	bUseAsOccluder = true;
	Mobility = EComponentMobility::Static;
	SetGenerateOverlapEvents(false);

	SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
}

#if WITH_EDITOR
void UModelComponent::InitializeModelComponent(UModel* InModel, uint16 InComponentIndex, uint32 MaskedSurfaceFlags, const TArray<uint16>& InNodes)
{
	Model = InModel;
	ComponentIndex = InComponentIndex;
	Nodes = InNodes;

	// Model components are transacted.
	SetFlags(RF_Transactional);

	CastShadow = true;
	bUseAsOccluder = true;
	Mobility = EComponentMobility::Static;
	SetGenerateOverlapEvents(false);

	SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
}
#endif // WITH_EDITOR


void UModelComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UModelComponent* This = CastChecked<UModelComponent>(InThis);
	Collector.AddReferencedObject( This->Model, This );
	for( int32 ElementIndex=0; ElementIndex < This->Elements.Num(); ElementIndex++ )
	{
		FModelElement& Element = This->Elements[ElementIndex];
		Collector.AddReferencedObject( Element.Component, This );
		Collector.AddReferencedObject( Element.Material, This );
	}
	Super::AddReferencedObjects( This, Collector );
}

void UModelComponent::CommitSurfaces()
{
	TArray<int32> InvalidElements;

	// Find nodes that are from surfaces which have been invalidated.
	TMap<uint16,FModelElement*> InvalidNodes;
	for(int32 ElementIndex = 0;ElementIndex < Elements.Num();ElementIndex++)
	{
		FModelElement& Element = Elements[ElementIndex];
		TArray<uint16> NewNodes;
		for(int32 NodeIndex = 0;NodeIndex < Element.Nodes.Num();NodeIndex++)
		{
			FBspNode& Node = Model->Nodes[Element.Nodes[NodeIndex]];
			FBspSurf& Surf = Model->Surfs[Node.iSurf];
			if(Surf.Material != Element.Material)
			{
				// This node's material changed, remove it from the element and put it on the invalid node list.
				InvalidNodes.Add(Element.Nodes[NodeIndex],&Element);

				// Mark the node's original element as being invalid.
				InvalidElements.AddUnique(ElementIndex);
			}
			else
			{
				NewNodes.Add(Element.Nodes[NodeIndex]);
			}
		}
		Element.Nodes = NewNodes;
	}

	// Reassign the invalid nodes to appropriate mesh elements.
	for(TMap<uint16,FModelElement*>::TConstIterator It(InvalidNodes);It;++It)
	{
		FBspNode& Node = Model->Nodes[It.Key()];
		FBspSurf& Surf = Model->Surfs[Node.iSurf];
		FModelElement* OldElement = It.Value();

		// Find an element which has the same material and lights as the invalid node.
		FModelElement* NewElement = NULL;
		for(int32 ElementIndex = 0;ElementIndex < Elements.Num();ElementIndex++)
		{
			FModelElement& Element = Elements[ElementIndex];
			if(Element.Material != Surf.Material)
				continue;
			if(Element.MapBuildDataId != OldElement->MapBuildDataId)
				continue;
			// This element's material and lights match the node.
			NewElement = &Element;
		}

		// If no matching element was found, create a new element.
		if(!NewElement)
		{
			NewElement = new FModelElement(this,Surf.Material);
			Elements.Add(NewElement);
			NewElement->MapBuildDataId = OldElement->MapBuildDataId;
		}

		NewElement->Nodes.Add(It.Key());
		InvalidElements.AddUnique(Elements.Num() - 1);
	}

	// Rebuild the render data for the elements which have changed.
	BuildRenderData();

	ShrinkElements();

#if WITH_EDITOR
	// Need to update collision data as well
	InvalidateCollisionData();
#endif	//WITH_EDITOR
}

void UModelComponent::ShrinkElements()
{
	// Find elements which have no nodes remaining.
	for(int32 ElementIndex = 0;ElementIndex < Elements.Num();ElementIndex++)
	{
		FModelElement& Element = Elements[ElementIndex];
		if(!Element.Nodes.Num())
		{
			// This element contains no nodes, remove it.
			Elements.RemoveAt(ElementIndex--);
		}
	}
}

void UModelComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	Ar << Model;

	if( Ar.IsLoading() && Ar.UEVer() < VER_UE4_REMOVE_ZONES_FROM_MODEL)
	{
		int32 DummyZoneIndex;
		Ar << DummyZoneIndex << Elements;
	}
	else
	{
		Ar << Elements;
	}

	if (Ar.IsLoading() && Elements.Num() > 0)
	{
		FMeshMapBuildLegacyData LegacyComponentData;

		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
		{
			FModelElement& Element = Elements[ElementIndex];

			if (Element.LegacyMapBuildData)
			{
				LegacyComponentData.Data.Emplace(Element.MapBuildDataId, Element.LegacyMapBuildData);
				Element.LegacyMapBuildData = NULL;
			}
		}

		if (LegacyComponentData.Data.Num() > 0)
		{
			GComponentsWithLegacyLightmaps.AddAnnotation(this, MoveTemp(LegacyComponentData));
		}
	}

	Ar << ComponentIndex << Nodes;
}

void UModelComponent::PostLoad()
{
	Super::PostLoad();

	// Fix for old model components which weren't created with transactional flag.
	SetFlags( RF_Transactional );

	// BuildRenderData relies on the model having been post-loaded, so we ensure this by calling ConditionalPostLoad.
	check(Model);
	Model->ConditionalPostLoad();

	// Initialize model elements' index buffers (required for generating ddc data).
	BuildRenderData();

	// Older content without a body setup
	if(ModelBodySetup == NULL)
	{
		CreateModelBodySetup();
		check(ModelBodySetup != NULL);
		ModelBodySetup->CreatePhysicsMeshes(); // Want to do this in PostLoad before model vertex buffer is discarded
	}

	// Stop existing ModelComponents from generating mirrored collision mesh
	if ((GetLinkerUEVersion() < VER_UE4_NO_MIRROR_BRUSH_MODEL_COLLISION) && (ModelBodySetup != NULL))
	{
		ModelBodySetup->bGenerateMirroredCollision = false;
	}

	ModelBodySetup->bDoubleSidedGeometry = true;	//saved content wants this to be true
}

#if WITH_EDITOR
void UModelComponent::PostEditUndo()
{
	// Rebuild the component's render data after applying a transaction to it.
	ULevel* Level = GetTypedOuter<ULevel>();
	if (ensure(Level))
	{
		Level->InvalidateModelSurface();
	}
	Super::PostEditUndo();
}
#endif // WITH_EDITOR

bool UModelComponent::IsNameStableForNetworking() const
{
	// UModelComponent is always persistent for the duration of a game session, and so can be considered to have a stable name
	return true;
}

void UModelComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	for( int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex )
	{
		const FModelElement& Element = Elements[ ElementIndex ];
		if( Element.Material )
		{
			OutMaterials.Add( Element.Material );
		}
	}
}

int32 UModelComponent::GetNumMaterials() const
{
	return Elements.Num();
}


UMaterialInterface* UModelComponent::GetMaterial(int32 MaterialIndex) const
{
	UMaterialInterface* Material = nullptr;

	if(Elements.IsValidIndex(MaterialIndex))
	{
		return  Elements[MaterialIndex].Material;
	}

	return Material;
}

UMaterialInterface* UModelComponent::GetMaterialFromCollisionFaceIndex(int32 FaceIndex, int32& SectionIndex) const
{
	UMaterialInterface* Result = nullptr;
	SectionIndex = 0;

	if (FaceIndex >= 0)
	{
		// Look for element that corresponds to the supplied face
		int32 TotalFaceCount = 0;
		for (int32 ElementIdx = 0; ElementIdx < Elements.Num(); ElementIdx++)
		{
			const FModelElement& Element = Elements[ElementIdx];
			TotalFaceCount += Element.NumTriangles;

			if (FaceIndex < TotalFaceCount)
			{
				// Grab the material
				Result = Element.Material;
				SectionIndex = ElementIdx;
				break;
			}
		}
	}

	return Result;
}

bool UModelComponent::IsPrecomputedLightingValid() const
{
	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
	{
		const FModelElement& Element = Elements[ElementIndex];

		if (Element.GetMeshMapBuildData() != NULL)
		{
			return true;
		}
	}

	return false;
}

#if WITH_EDITOR
void UModelComponent::SelectAllSurfaces()
{
	check(Model);
	for( int32 NodeIndex=0; NodeIndex<Nodes.Num(); NodeIndex++ )
	{
		FBspNode& Node = Model->Nodes[Nodes[NodeIndex]];
		FBspSurf& Surf = Model->Surfs[Node.iSurf];
		Model->ModifySurf( Node.iSurf, 0 );
		Surf.PolyFlags |= PF_Selected;
	}
}
#endif // WITH_EDITOR

void UModelComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	if( Model )
	{
		// Build a map from surface indices to node indices for nodes in this component.
		TMultiMap<int32,int32> SurfToNodeMap;
		for(int32 NodeIndex = 0;NodeIndex < Nodes.Num();NodeIndex++)
		{
			const FBspNode& Node = Model->Nodes[Nodes[NodeIndex]];
			SurfToNodeMap.Add(Node.iSurf,Nodes[NodeIndex]);
		}

		TArray<int32> SurfaceNodes;
		TArray<FVector> SurfaceVertices;
		for(int32 SurfaceIndex = 0;SurfaceIndex < Model->Surfs.Num();SurfaceIndex++)
		{
			// Check if the surface has nodes in this component.
			SurfaceNodes.Reset();
			SurfToNodeMap.MultiFind(SurfaceIndex,SurfaceNodes);
			if(SurfaceNodes.Num())
			{
				const FBspSurf& Surf = Model->Surfs[SurfaceIndex];

				// Compute a bounding sphere for the surface's nodes.
				SurfaceVertices.Reset();
				for(int32 NodeIndex = 0;NodeIndex < SurfaceNodes.Num();NodeIndex++)
				{
					const FBspNode& Node = Model->Nodes[SurfaceNodes[NodeIndex]];
					for(int32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
					{
						const FVector WorldVertex = GetComponentTransform().TransformPosition((FVector)Model->Points[Model->Verts[Node.iVertPool + VertexIndex].pVertex]);
						SurfaceVertices.Add(WorldVertex);
					}
				}
				const FSphere SurfaceBoundingSphere(SurfaceVertices.GetData(),SurfaceVertices.Num());

				// Compute the surface's texture scaling factor.
				const float BspTexelsPerNormalizedTexel = UModel::GetGlobalBSPTexelScale();
				const float WorldUnitsPerBspTexel = FMath::Max( Model->Vectors[Surf.vTextureU].Size(), Model->Vectors[Surf.vTextureV].Size() );
				const float TexelFactor = BspTexelsPerNormalizedTexel / WorldUnitsPerBspTexel;

				// Determine the material applied to the surface.
				UMaterialInterface* Material = Surf.Material;
				if(!Material)
				{
					Material = UMaterial::GetDefaultMaterial(MD_Surface);
				}

				// Enumerate the textures used by the surface's material.
				TArray<UTexture*> Textures;
				
				Material->GetUsedTextures(Textures, GetCurrentMaterialQualityLevelChecked());

				// Add each texture to the output with the appropriate parameters.
				for(int32 TextureIndex = 0;TextureIndex < Textures.Num();TextureIndex++)
				{
					UTexture2D* Texture2D = Cast<UTexture2D>(Textures[TextureIndex]);
					if (!Texture2D) continue;

					FStreamingRenderAssetPrimitiveInfo& StreamingTexture = *new(OutStreamingRenderAssets) FStreamingRenderAssetPrimitiveInfo;
					StreamingTexture.Bounds = SurfaceBoundingSphere;
					StreamingTexture.TexelFactor = TexelFactor;
					StreamingTexture.RenderAsset = Texture2D;
				}
			}
		}
	}
}

#if WITH_EDITOR

struct FNodeGroupKey
{
	FNodeGroup* NodeGroup;
	uint32 LightMapScale;
	UMaterialInterface* MaterialInterface;

	FNodeGroupKey(FNodeGroup* InNodeGroup, float InLightMapScale, UMaterialInterface* InMaterialInterface)
		: NodeGroup(InNodeGroup)
		, LightMapScale(FMath::TruncToInt(InLightMapScale))
		, MaterialInterface(InMaterialInterface)
	{}

	friend FORCEINLINE bool operator == (const FNodeGroupKey& A, const FNodeGroupKey& B)
	{
		return A.NodeGroup == B.NodeGroup && A.LightMapScale == B.LightMapScale && A.MaterialInterface == B.MaterialInterface;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FNodeGroupKey& Key)
	{
		return HashCombine(GetTypeHash(Key.NodeGroup), HashCombine(Key.LightMapScale, GetTypeHash(Key.MaterialInterface)));
	}
};

bool UModelComponent::GenerateElements(bool bBuildRenderData)
{
	Elements.Empty();
	if (GIsEditor == false)
	{
		TMap<UMaterialInterface*,FModelElement*> MaterialToElementMap;

		// Find the BSP nodes which are in this component's zone.
		for(int32 NodeIndex = 0;NodeIndex < Nodes.Num();NodeIndex++)
		{
			FBspNode& Node = Model->Nodes[Nodes[NodeIndex]];
			FBspSurf& Surf = Model->Surfs[Node.iSurf];

			// Find an element with the same material as this node.
			FModelElement* Element = MaterialToElementMap.FindRef(Surf.Material);
			if(!Element)
			{
				// If there's no matching element, create a new element.
				FModelElement* NewElement = new FModelElement(this,Surf.Material);
				Elements.Add(NewElement);
				Element = MaterialToElementMap.Add(Surf.Material, NewElement);
			}

			// Add the node to the element.
			Element->Nodes.Add(Nodes[NodeIndex]);
		}
	}
	else
	{
		// Prebuild an array relating the node index to the corresponding NodeGroup
		TArray<FNodeGroup*> NodeGroupForNode;
		NodeGroupForNode.AddZeroed(Model->Nodes.Num());
		for (const auto& NodeGroupPair : Model->NodeGroups)
		{
			FNodeGroup* NodeGroup = NodeGroupPair.Value;

			for (int32 Node : NodeGroup->Nodes)
			{
				check(NodeGroupForNode[Node] == nullptr);
				NodeGroupForNode[Node] = NodeGroup;
			}
		}

		TMap<FNodeGroupKey, FModelElement*> NodeGroupToElementMap;
		NodeGroupToElementMap.Empty(Nodes.Num());

		// Find the BSP nodes which are in this component's zone.
		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
		{
			FBspNode& Node = Model->Nodes[Nodes[NodeIndex]];
			FBspSurf& Surf = Model->Surfs[Node.iSurf];

			// Find the node group it belong to
			FNodeGroup* FoundNodeGroup = NodeGroupForNode[Nodes[NodeIndex]];

			const FNodeGroupKey NodeGroupKey(FoundNodeGroup, Surf.LightMapScale, Surf.Material);
			FModelElement** Element = NodeGroupToElementMap.Find(NodeGroupKey);
			if (Element == nullptr)
			{
				FModelElement* NewElement = new FModelElement(this, Surf.Material);
				Elements.Add(NewElement);
				Element = &NodeGroupToElementMap.Add(NodeGroupKey, NewElement);
			}
			check(Element);

			// Add the node to the element.
			(*Element)->Nodes.Add(Nodes[NodeIndex]);
		}
	}

	if (bBuildRenderData == true)
	{
		BuildRenderData();
	}

	return true;
}
#endif // WITH_EDITOR

void UModelComponent::CopyElementsFrom(UModelComponent* SrcComponent)
{
	Elements.Empty();
	for (int32 ElementIndex = 0; ElementIndex < SrcComponent->Elements.Num(); ++ElementIndex)
	{
		FModelElement& SrcElement = SrcComponent->Elements[ElementIndex];
		FModelElement& DestElement = *new FModelElement(SrcElement);
		Elements.Add(&DestElement);
		DestElement.Component = this;
	}

	if(ModelBodySetup && SrcComponent->ModelBodySetup)
	{
		ModelBodySetup->BodySetupGuid = SrcComponent->ModelBodySetup->BodySetupGuid;
	}
}

void UModelComponent::CreateModelBodySetup()
{
	if(ModelBodySetup == NULL)
	{
		ModelBodySetup = NewObject<UBodySetup>(this);
		check(ModelBodySetup);
		ModelBodySetup->BodySetupGuid = FGuid::NewGuid();
	}

	ModelBodySetup->CollisionTraceFlag = CTF_UseComplexAsSimple;
	ModelBodySetup->bGenerateMirroredCollision = false;
	ModelBodySetup->bDoubleSidedGeometry = true;
}

#if WITH_EDITOR
void UModelComponent::InvalidateCollisionData()
{
	// Make sure we have a body setup..
	CreateModelBodySetup(); 
	check(ModelBodySetup != NULL);

	UE_LOGF(LogPhysics, Log, "Invalidate ModelComponent: %ls", *GetPathName());

	// Then give it a new GUID
	ModelBodySetup->InvalidatePhysicsData();
}
#endif // WITH_EDITOR

bool UModelComponent::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const
{
	check(Model);
	OutTriMeshEstimates.VerticeCount = Model->VertexBuffer.Vertices.Num();
	return true;
}

bool UModelComponent::GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData)
{
	check(Model);
	int32 nBadArea = 0;
	const float AreaThreshold = UPhysicsSettings::Get()->TriangleMeshTriangleMinAreaThreshold;

	// See if we should copy UVs
	bool bCopyUVs = UPhysicsSettings::Get()->bSupportUVFromHitResults;
	if (bCopyUVs)
	{
		CollisionData->UVs.AddZeroed(1); // only one UV channel
	}

	const int32 NumVerts = Model->VertexBuffer.Vertices.Num();
	CollisionData->Vertices.AddUninitialized(NumVerts);
	if (bCopyUVs)
	{
		CollisionData->UVs[0].AddUninitialized(NumVerts);
	}
	for(int32 i=0; i<NumVerts; i++)
	{
		CollisionData->Vertices[i] = Model->VertexBuffer.Vertices[i].Position;
		if (bCopyUVs)
		{
			CollisionData->UVs[0][i] = FVector2D(Model->VertexBuffer.Vertices[i].TexCoord);
		}
	}

	for(int32 ElementIndex = 0;ElementIndex < Elements.Num();ElementIndex++)
	{
		FModelElement& Element = Elements[ElementIndex];
		if (Element.NumTriangles == 0)
		{
			// CSG can produce empty elements, just ignore them.
			continue;
		}

		FRawIndexBuffer16or32* IndexBuffer = Element.IndexBuffer;
		if (IndexBuffer == nullptr)
		{
			UE_LOGF(LogPhysics, Warning, "Found NULL index buffer when cooking UModelComponent physics data! Component: %ls, Element: %d, NumTriangles: %d.", *GetPathName(), ElementIndex, Element.NumTriangles);
			continue;
		}

		int32 NumIndices = IndexBuffer->Indices.Num();
		int32 NumVertices = Model->VertexBuffer.Vertices.Num();

		if (NumVertices < (int32)Element.MaxVertexIndex)
		{
			UE_LOGF(LogPhysics, Warning, "Found bad vertex buffer when cooking UModelComponent physics data! Component: %ls, Element: %d. Expected Vertex Count: %d, Actual Vertex Count: %d",
				*GetPathName(), ElementIndex, Element.MaxVertexIndex, NumVertices);
			continue;
		}

		if (NumIndices < (int32)(Element.FirstIndex + Element.NumTriangles * 3 - 1))
		{
			UE_LOGF(LogPhysics, Warning, "Found bad index buffer when cooking UModelComponent physics data! Not enough indices in buffer for Element triangles. Component: %ls, Element: %d, Index Start: %d, Tri Count: %d, Index Count: %d",
				*GetPathName(), ElementIndex, Element.FirstIndex, Element.NumTriangles, NumIndices);
			continue;
		}

		for (uint32 TriIdx = 0; TriIdx < Element.NumTriangles; TriIdx++)
		{
			FTriIndices Triangle;

			Triangle.v0 = IndexBuffer->Indices[Element.FirstIndex + (TriIdx * 3) + 0];
			Triangle.v1 = IndexBuffer->Indices[Element.FirstIndex + (TriIdx * 3) + 1];
			Triangle.v2 = IndexBuffer->Indices[Element.FirstIndex + (TriIdx * 3) + 2];

			if (AreaThreshold >= 0.f)
			{
				const FVector V0 = (FVector)Model->VertexBuffer.Vertices[Triangle.v0].Position;
				const FVector V1 = (FVector)Model->VertexBuffer.Vertices[Triangle.v1].Position;
				const FVector V2 = (FVector)Model->VertexBuffer.Vertices[Triangle.v2].Position;

				const FVector V01 = (V1 - V0);
				const FVector V02 = (V2 - V0);
				const FVector Cross = FVector::CrossProduct(V01, V02);
				const float Area = Cross.Size() * 0.5f;
				if (Area <= AreaThreshold)
				{
					nBadArea++;
					continue;
				}
			}

			CollisionData->Indices.Add(Triangle);
			CollisionData->MaterialIndices.Add(ElementIndex);
		}
	}

	if (nBadArea > 0)
	{
		UE_LOGF(LogPhysics, Log, "Cooking removed %d triangle%ls with area <= %f (%ls)", nBadArea, (nBadArea > 1 ? TEXT("s") : TEXT("")), AreaThreshold, *GetPathName(GetOuter()));
	}

	CollisionData->bFlipNormals = true;
	return true;
}

bool UModelComponent::ContainsPhysicsTriMeshData(bool InUseAllTriData) const
{
	return (Elements.Num() > 0);
}

