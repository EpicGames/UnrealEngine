// Copyright Epic Games, Inc. All Rights Reserved.


#include "GameFramework/MovementComponent.h"
#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/PhysicsVolume.h"
#include "Logging/MessageLog.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementComponent)

#define LOCTEXT_NAMESPACE "MovementComponent"
DEFINE_LOG_CATEGORY_STATIC(LogMovement, Log, All);

namespace MovementComponentStatics
{
	static const FName TestOverlapName = FName("MovementOverlapTest");
}

//----------------------------------------------------------------------//
// UMovementComponent
//----------------------------------------------------------------------//
UMovementComponent::UMovementComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bCanEverTick = true;

	MoveComponentFlags = MOVECOMP_NoFlags;

	bUpdateOnlyIfRendered = false;
	bAutoUpdateTickRegistration = true;
	bTickBeforeOwner = true;
	bAutoRegisterUpdatedComponent = true;
	bAutoRegisterPhysicsVolumeUpdates = true;
	bComponentShouldUpdatePhysicsVolume = false;

	PlaneConstraintNormal = FVector::ZeroVector;
	PlaneConstraintAxisSetting = EPlaneConstraintAxisSetting::Custom;
	bConstrainToPlane = false;
	bSnapToPlaneAtStart = false;

	bWantsInitializeComponent = true;
	bAutoActivate = true;
	bInOnRegister = false;
	bInInitializeComponent = false;
}


void UMovementComponent::SetUpdatedComponent(USceneComponent* NewUpdatedComponent)
{
	if (UpdatedComponent && UpdatedComponent != NewUpdatedComponent)
	{
		if (bAutoRegisterPhysicsVolumeUpdates)
		{
			UpdatedComponent->SetShouldUpdatePhysicsVolume(false);
			if (IsValid(UpdatedComponent))
			{
				UpdatedComponent->SetPhysicsVolume(NULL, true);
			}
		}
		UpdatedComponent->PhysicsVolumeChangedDelegate.RemoveDynamic(this, &UMovementComponent::PhysicsVolumeChanged);

		// remove from tick prerequisite
		UpdatedComponent->PrimaryComponentTick.RemovePrerequisite(this, PrimaryComponentTick); 
	}

	// Don't assign pending kill components, but allow those to null out previous UpdatedComponent.
	UpdatedComponent = IsValid(NewUpdatedComponent) ? NewUpdatedComponent : NULL;
	UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);

	// Assign delegates
	if (UpdatedComponent)
	{
		// Listen to events regardless of whether enabled, in case physics volume updates are later enabled.
		UpdatedComponent->PhysicsVolumeChangedDelegate.AddUniqueDynamic(this, &UMovementComponent::PhysicsVolumeChanged);

		// Handle auto registration
		if (bAutoRegisterPhysicsVolumeUpdates)
		{
			UpdatedComponent->SetShouldUpdatePhysicsVolume(bComponentShouldUpdatePhysicsVolume);
			if (bComponentShouldUpdatePhysicsVolume)
			{
				if (!bInOnRegister && !bInInitializeComponent)
				{
					// UpdateOverlaps() in component registration will take care of this.
					UpdatedComponent->UpdatePhysicsVolume(true);
				}
			}
			else
			{
				UpdatedComponent->SetPhysicsVolume(NULL, true);
			}
		}
		
		// force ticks after movement component updates
		UpdatedComponent->PrimaryComponentTick.AddPrerequisite(this, PrimaryComponentTick); 
	}

	UpdateTickRegistration();

	if (bSnapToPlaneAtStart)
	{
		SnapUpdatedComponentToPlane();
	}
}


void UMovementComponent::InitializeComponent()
{
	TGuardValue<bool> InInitializeComponentGuard(bInInitializeComponent, true);
	Super::InitializeComponent();

	// RootComponent is null in OnRegister for blueprint (non-native) root components.
	if (bAutoRegisterUpdatedComponent && !IsValid(UpdatedComponent))
	{
		// Auto-register owner's root component if found.
		if (AActor* MyActor = GetOwner())
		{
			if (USceneComponent* NewUpdatedComponent = MyActor->GetRootComponent())
			{
				SetUpdatedComponent(NewUpdatedComponent);
			}
		}
	}

	if (bSnapToPlaneAtStart)
	{
		SnapUpdatedComponentToPlane();
	}
}


void UMovementComponent::OnRegister()
{
	TGuardValue<bool> InOnRegisterGuard(bInOnRegister, true);

	UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
	Super::OnRegister();

	if (PlaneConstraintAxisSetting != EPlaneConstraintAxisSetting::Custom)
	{
		SetPlaneConstraintAxisSetting(PlaneConstraintAxisSetting);
	}

	const UWorld* MyWorld = GetWorld();
	if (MyWorld && (MyWorld->IsGameWorld()
#if WITH_EDITOR
	|| MyWorld->bForceUseMovementComponentInNonGameWorld
#endif
	))
	{
		PlaneConstraintNormal = PlaneConstraintNormal.GetSafeNormal();

		USceneComponent* NewUpdatedComponent = UpdatedComponent;
		if (!UpdatedComponent && bAutoRegisterUpdatedComponent)
		{
			// Auto-register owner's root component if found.
			AActor* MyActor = GetOwner();
			if (MyActor)
			{
				NewUpdatedComponent = MyActor->GetRootComponent();
			}
		}

		SetUpdatedComponent(NewUpdatedComponent);
	}

#if WITH_EDITOR
	// Reset so next PIE session warns.
	bEditorWarnedStaticMobilityMove = false;
#endif
}

void UMovementComponent::RegisterComponentTickFunctions(bool bRegister)
{
	Super::RegisterComponentTickFunctions(bRegister);

	// Super may start up the tick function when we don't want to.
	UpdateTickRegistration();

	// If the owner ticks, make sure we tick first
	AActor* Owner = GetOwner();
	if (bTickBeforeOwner && bRegister && PrimaryComponentTick.bCanEverTick && Owner && Owner->CanEverTick())
	{
		Owner->PrimaryActorTick.AddPrerequisite(this, PrimaryComponentTick);
	}
}

void UMovementComponent::UpdateTickRegistration()
{
	if (bAutoUpdateTickRegistration)
	{
		const bool bHasUpdatedComponent = (UpdatedComponent != NULL);
		SetComponentTickEnabled(bHasUpdatedComponent && bAutoActivate);
	}
}


void UMovementComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// Don't hang on to stale references to a destroyed UpdatedComponent.
	if (UpdatedComponent != NULL && !IsValid(UpdatedComponent))
	{
		SetUpdatedComponent(NULL);
	}
}


void UMovementComponent::Serialize(FArchive& Ar)
{
	USceneComponent* CurrentUpdatedComponent = UpdatedComponent;
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		// This was marked Transient so it won't be saved out, but we need still to reject old saved values.
		UpdatedComponent = CurrentUpdatedComponent;
		UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
	}
}

void UMovementComponent::PostLoad()
{
	Super::PostLoad();

	if (PlaneConstraintAxisSetting == EPlaneConstraintAxisSetting::UseGlobalPhysicsSetting)
	{
		// Make sure to use the most up-to-date project setting in case it has changed.
		PlaneConstraintNormal = GetPlaneConstraintNormalFromAxisSetting(PlaneConstraintAxisSetting);
	}

	UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
}


void UMovementComponent::Deactivate()
{
	Super::Deactivate();
	if (!IsActive())
	{
		StopMovementImmediately();
	}
}

#if WITH_EDITOR
void UMovementComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	if (PropertyThatChanged)
	{
		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UMovementComponent, PlaneConstraintAxisSetting))
		{
			PlaneConstraintNormal = GetPlaneConstraintNormalFromAxisSetting(PlaneConstraintAxisSetting);
		}
		else if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UMovementComponent, PlaneConstraintNormal))
		{
			PlaneConstraintAxisSetting = EPlaneConstraintAxisSetting::Custom;
		}
	}
}

void UMovementComponent::PhysicsLockedAxisSettingChanged()
{
	
	for (TObjectIterator<UMovementComponent> Iter; Iter; ++Iter)
	{
		UMovementComponent* MovementComponent = *Iter;
		if (MovementComponent->PlaneConstraintAxisSetting == EPlaneConstraintAxisSetting::UseGlobalPhysicsSetting)
		{
			MovementComponent->PlaneConstraintNormal = MovementComponent->GetPlaneConstraintNormalFromAxisSetting(MovementComponent->PlaneConstraintAxisSetting);
		}
	}
}

#endif // WITH_EDITOR


void UMovementComponent::PhysicsVolumeChanged(APhysicsVolume* NewVolume)
{
	//UE_LOGF(LogMovement, Log, "PhysicsVolumeChanged : No implementation");
}

APhysicsVolume* UMovementComponent::GetPhysicsVolume() const
{
	if (UpdatedComponent)
	{
		return UpdatedComponent->GetPhysicsVolume();
	}
	
	return GetWorld()->GetDefaultPhysicsVolume();
}

bool UMovementComponent::IsInWater() const
{
	const APhysicsVolume* PhysVolume = GetPhysicsVolume();
	return PhysVolume && PhysVolume->bWaterVolume;
}

bool UMovementComponent::ShouldSkipUpdate(float DeltaTime) const
{
	if (UpdatedComponent == nullptr)
	{
		return true;
	}
		
	if (UpdatedComponent->Mobility != EComponentMobility::Movable)
	{
#if WITH_EDITOR
		if (!bEditorWarnedStaticMobilityMove)
		{
			if (UWorld * World = GetWorld())
			{
				if (World->HasBegunPlay() && IsRegistered())
				{
					const_cast<UMovementComponent*>(this)->bEditorWarnedStaticMobilityMove = true;
					FMessageLog("PIE").Warning(FText::Format(LOCTEXT("InvalidMove", "Mobility of {0} : {1} has to be 'Movable' if you'd like to move it with {2}. "),
						FText::FromString(GetPathNameSafe(UpdatedComponent->GetOwner())), FText::FromString(UpdatedComponent->GetName()), FText::FromString(GetClass()->GetName())));
				}
			}
		}
#endif

		return true;
	}

	if (bUpdateOnlyIfRendered)
	{
		if (IsNetMode(NM_DedicatedServer))
		{
			// Dedicated servers never render
			return true;
		}

		const float RenderTimeThreshold = 0.41f;
		UWorld* TheWorld = GetWorld();
		if (UpdatedPrimitive && TheWorld->TimeSince(UpdatedPrimitive->GetLastRenderTime()) <= RenderTimeThreshold)
		{
			return false; // Rendered, don't skip it.
		}

		// Most components used with movement components don't actually render, so check attached children render times.
		TArray<USceneComponent*> Children;
		UpdatedComponent->GetChildrenComponents(true, Children);
		for (auto Child : Children)
		{
			const UPrimitiveComponent* PrimitiveChild = Cast<UPrimitiveComponent>(Child);
			if (PrimitiveChild)
			{
				if (PrimitiveChild->IsRegistered() && TheWorld->TimeSince(PrimitiveChild->GetLastRenderTime()) <= RenderTimeThreshold)
				{
					return false; // Rendered, don't skip it.
				}
			}
		}

		// No children were recently rendered, safely skip the update.
		return true;
	}

	return false;
}


float UMovementComponent::GetGravityZ() const
{
	APhysicsVolume* PhysicsVolume = GetPhysicsVolume();
	return PhysicsVolume ? PhysicsVolume->GetGravityZ() : UPhysicsSettings::Get()->DefaultGravityZ;
}

void UMovementComponent::HandleImpact(const FHitResult& Hit, float TimeSlice, const FVector& MoveDelta)
{
	
}

void UMovementComponent::UpdateComponentVelocity()
{
	if ( UpdatedComponent )
	{
		UpdatedComponent->ComponentVelocity = Velocity;
	}
}

void UMovementComponent::InitCollisionParams(FCollisionQueryParams &OutParams, FCollisionResponseParams& OutResponseParam) const
{
	if (UpdatedPrimitive)
	{
		UpdatedPrimitive->InitSweepCollisionParams(OutParams, OutResponseParam);
	}
}

bool UMovementComponent::OverlapTest(const FVector& Location, const FQuat& RotationQuat, const ECollisionChannel CollisionChannel, const FCollisionShape& CollisionShape, const AActor* IgnoreActor) const
{
	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(MovementOverlapTest), false, IgnoreActor);
	FCollisionResponseParams ResponseParam;
	InitCollisionParams(QueryParams, ResponseParam);
	return GetWorld()->OverlapBlockingTestByChannel(Location, RotationQuat, CollisionChannel, CollisionShape, QueryParams, ResponseParam);
}

bool UMovementComponent::IsExceedingMaxSpeed(float MaxSpeed) const
{
	MaxSpeed = FMath::Max(0.f, MaxSpeed);
	const float MaxSpeedSquared = FMath::Square(MaxSpeed);
	
	// Allow 1% error tolerance, to account for numeric imprecision.
	const float OverVelocityPercent = 1.01f;
	return (Velocity.SizeSquared() > MaxSpeedSquared * OverVelocityPercent);
}


FVector UMovementComponent::GetPlaneConstraintNormalFromAxisSetting(EPlaneConstraintAxisSetting AxisSetting) const
{
	if (AxisSetting == EPlaneConstraintAxisSetting::UseGlobalPhysicsSetting)
	{
		ESettingsDOF::Type GlobalSetting = UPhysicsSettings::Get()->DefaultDegreesOfFreedom;
		switch (GlobalSetting)
		{
		case ESettingsDOF::Full3D:	return FVector::ZeroVector;
		case ESettingsDOF::YZPlane:	return FVector(1.f, 0.f, 0.f);
		case ESettingsDOF::XZPlane:	return FVector(0.f, 1.f, 0.f);
		case ESettingsDOF::XYPlane:	return FVector(0.f, 0.f, 1.f);
		default:
			checkf(false, TEXT("GetPlaneConstraintNormalFromAxisSetting: Unknown global axis setting %d for %s"), int32(GlobalSetting), *GetNameSafe(GetOwner()));
			return FVector::ZeroVector;
		}
	}

	switch(AxisSetting)
	{
	case EPlaneConstraintAxisSetting::Custom:	return PlaneConstraintNormal;
	case EPlaneConstraintAxisSetting::X:		return FVector(1.f, 0.f, 0.f);
	case EPlaneConstraintAxisSetting::Y:		return FVector(0.f, 1.f, 0.f);
	case EPlaneConstraintAxisSetting::Z:		return FVector(0.f, 0.f, 1.f);
	default:
		checkf(false, TEXT("GetPlaneConstraintNormalFromAxisSetting: Unknown axis %d for %s"), int32(AxisSetting), *GetNameSafe(GetOwner()));
		return FVector::ZeroVector;
	}
}

void UMovementComponent::SetPlaneConstraintAxisSetting(EPlaneConstraintAxisSetting NewAxisSetting)
{
	PlaneConstraintAxisSetting = NewAxisSetting;
	PlaneConstraintNormal = GetPlaneConstraintNormalFromAxisSetting(PlaneConstraintAxisSetting);
}

void UMovementComponent::SetPlaneConstraintNormal(FVector PlaneNormal)
{
	PlaneConstraintNormal = PlaneNormal.GetSafeNormal();
	PlaneConstraintAxisSetting = EPlaneConstraintAxisSetting::Custom;
}

void UMovementComponent::SetPlaneConstraintFromVectors(FVector Forward, FVector Up)
{
	PlaneConstraintNormal = (Up ^ Forward).GetSafeNormal();
}

void UMovementComponent::SetPlaneConstraintOrigin(FVector PlaneOrigin)
{
	PlaneConstraintOrigin = PlaneOrigin;
}

void UMovementComponent::SetPlaneConstraintEnabled(bool bEnabled)
{
	bConstrainToPlane = bEnabled;
}

const FVector& UMovementComponent::GetPlaneConstraintOrigin() const
{
	return PlaneConstraintOrigin;
}

const FVector& UMovementComponent::GetPlaneConstraintNormal() const
{
	return PlaneConstraintNormal;
}

FVector UMovementComponent::ConstrainDirectionToPlane(FVector Direction) const
{
	if (bConstrainToPlane)
	{
		Direction = FVector::VectorPlaneProject(Direction, PlaneConstraintNormal);
	}

	return Direction;
}


FVector UMovementComponent::ConstrainLocationToPlane(FVector Location) const
{
	if (bConstrainToPlane)
	{
		Location = FVector::PointPlaneProject(Location, PlaneConstraintOrigin, PlaneConstraintNormal);
	}

	return Location;
}


FVector UMovementComponent::ConstrainNormalToPlane(FVector Normal) const
{
	if (bConstrainToPlane)
	{
		Normal = FVector::VectorPlaneProject(Normal, PlaneConstraintNormal).GetSafeNormal();
	}

	return Normal;
}


void UMovementComponent::SnapUpdatedComponentToPlane()
{
	if (UpdatedComponent && bConstrainToPlane)
	{
		UpdatedComponent->SetWorldLocation( ConstrainLocationToPlane(UpdatedComponent->GetComponentLocation()) );
	}
}



bool UMovementComponent::MoveUpdatedComponentImpl( const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* OutHit, ETeleportType Teleport)
{
	if (UpdatedComponent)
	{
		const FVector NewDelta = ConstrainDirectionToPlane(Delta);
		return UpdatedComponent->MoveComponent(NewDelta, NewRotation, bSweep, OutHit, MoveComponentFlags, Teleport);
	}

	return false;
}


bool UMovementComponent::K2_MoveUpdatedComponent(FVector Delta, FRotator NewRotation, FHitResult& OutHit, bool bSweep, bool bTeleport)
{
	return SafeMoveUpdatedComponent(Delta, NewRotation.Quaternion(), bSweep, OutHit, TeleportFlagToEnum(bTeleport));
}

namespace MovementComponentCVars
{
// Typically we want to depenetrate regardless of direction, so we can get all the way out of penetration quickly.
// Our rules for "moving with depenetration normal" only get us so far out of the object. We'd prefer to pop out by the full MTD amount.
// Depenetration moves (in ResolvePenetration) then ignore blocking overlaps to be able to move out by the MTD amount.
int32 MoveIgnoreFirstBlockingOverlap = 0;
static FAutoConsoleVariableRef CVarMoveIgnoreFirstBlockingOverlap(
	TEXT("p.MoveIgnoreFirstBlockingOverlap"),
	MoveIgnoreFirstBlockingOverlap,
	TEXT("Whether to ignore the first blocking overlap in SafeMoveUpdatedComponent (if moving out from object and starting in penetration).\n")
	TEXT("The 'p.InitialOverlapTolerance' setting determines the 'move out' rules, but by default we always try to depenetrate first (not ignore the hit).\n")
	TEXT("0: Disable (do not ignore), 1: Enable (ignore)"),
	ECVF_Default);
}

bool UMovementComponent::SafeMoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport)
{
	if (UpdatedComponent == NULL)
	{
		OutHit.Reset(1.f);
		return false;
	}

	bool bMoveResult = false;

	// Scope for move flags
	{
		// Conditionally ignore blocking overlaps (based on CVar)
		const EMoveComponentFlags IncludeBlockingOverlapsWithoutEvents = (MOVECOMP_NeverIgnoreBlockingOverlaps | MOVECOMP_DisableBlockingOverlapDispatch);
		TGuardValue<EMoveComponentFlags> ScopedFlagRestore(MoveComponentFlags, MovementComponentCVars::MoveIgnoreFirstBlockingOverlap ? MoveComponentFlags : (MoveComponentFlags | IncludeBlockingOverlapsWithoutEvents));
		bMoveResult = MoveUpdatedComponent(Delta, NewRotation, bSweep, &OutHit, Teleport);
	}

	// Handle initial penetrations
	if (OutHit.bStartPenetrating && UpdatedComponent)
	{
		const FVector RequestedAdjustment = GetPenetrationAdjustment(OutHit);
		if (ResolvePenetration(RequestedAdjustment, OutHit, NewRotation))
		{
			// Retry original move
			bMoveResult = MoveUpdatedComponent(Delta, NewRotation, bSweep, &OutHit, Teleport);
		}
	}

	return bMoveResult;
}


namespace MovementComponentCVars
{
float PenetrationPullbackDistance = 0.125f;
static FAutoConsoleVariableRef CVarPenetrationPullbackDistance(TEXT("p.PenetrationPullbackDistance"),
	PenetrationPullbackDistance,
	TEXT("Pull out from penetration of an object by this extra distance.\n")
	TEXT("Distance added to penetration fix-ups."),
	ECVF_Default);

float PenetrationOverlapCheckInflation = 0.100f;
static FAutoConsoleVariableRef CVarPenetrationOverlapCheckInflation(TEXT("p.PenetrationOverlapCheckInflation"),
	PenetrationOverlapCheckInflation,
	TEXT("Inflation added to object when checking if a location is free of blocking collision.\n")
	TEXT("Distance added to inflation in penetration overlap check."),
	ECVF_Default);
}

FVector UMovementComponent::GetPenetrationAdjustment(const FHitResult& Hit) const
{
	if (!Hit.bStartPenetrating)
	{
		return FVector::ZeroVector;
	}

	FVector Result;
	const float PullBackDistance = FMath::Abs(MovementComponentCVars::PenetrationPullbackDistance);
	const float PenetrationDepth = (Hit.PenetrationDepth > 0.f ? Hit.PenetrationDepth : 0.125f);

	Result = Hit.Normal * (PenetrationDepth + PullBackDistance);

	return ConstrainDirectionToPlane(Result);
}

bool UMovementComponent::ResolvePenetrationImpl(const FVector& ProposedAdjustment, const FHitResult& Hit, const FQuat& NewRotationQuat)
{
	// SceneComponent can't be in penetration, so this function really only applies to PrimitiveComponent.
	const FVector Adjustment = ConstrainDirectionToPlane(ProposedAdjustment);
	if (!Adjustment.IsZero() && UpdatedPrimitive)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_MovementComponent_ResolvePenetration);
		// See if we can fit at the adjusted location without overlapping anything.
		AActor* ActorOwner = UpdatedComponent->GetOwner();
		if (!ActorOwner)
		{
			return false;
		}

		UE_LOGF(LogMovement, Verbose, "ResolvePenetration: %ls.%ls at location %ls inside %ls.%ls at location %ls by %.3f (netmode: %d)",
			   *ActorOwner->GetName(),
			   *UpdatedComponent->GetName(),
			   *UpdatedComponent->GetComponentLocation().ToString(),
			   *Hit.GetHitObjectHandle().GetName(),
			   *GetNameSafe(Hit.GetComponent()),
			   Hit.Component.IsValid() ? *Hit.GetComponent()->GetComponentLocation().ToString() : TEXT("<unknown>"),
			   Hit.PenetrationDepth,
			   (uint32)GetNetMode());

		// We really want to make sure that precision differences or differences between the overlap test and sweep tests don't put us into another overlap,
		// so make the overlap test a bit more restrictive.
		const float OverlapInflation = MovementComponentCVars::PenetrationOverlapCheckInflation;
		bool bEncroached = OverlapTest(Hit.TraceStart + Adjustment, NewRotationQuat, UpdatedPrimitive->GetCollisionObjectType(), UpdatedPrimitive->GetCollisionShape(OverlapInflation), ActorOwner);
		if (!bEncroached)
		{
			// Move without sweeping.
			MoveUpdatedComponent(Adjustment, NewRotationQuat, false, nullptr, ETeleportType::TeleportPhysics);
			UE_LOGF(LogMovement, Verbose, "ResolvePenetration:   teleport by %ls", *Adjustment.ToString());
			return true;
		}
		else
		{
			// Disable MOVECOMP_NeverIgnoreBlockingOverlaps if it is enabled, otherwise we wouldn't be able to sweep out of the object to fix the penetration.
			TGuardValue<EMoveComponentFlags> ScopedFlagRestore(MoveComponentFlags, EMoveComponentFlags(MoveComponentFlags & (~MOVECOMP_NeverIgnoreBlockingOverlaps)));

			// Try sweeping as far as possible...
			FHitResult SweepOutHit(1.f);
			bool bMoved = MoveUpdatedComponent(Adjustment, NewRotationQuat, true, &SweepOutHit, ETeleportType::TeleportPhysics);
			UE_LOGF(LogMovement, Verbose, "ResolvePenetration:   sweep by %ls (success = %d)", *Adjustment.ToString(), bMoved);
			
			// Still stuck?
			if (!bMoved && SweepOutHit.bStartPenetrating)
			{
				// Combine two MTD results to get a new direction that gets out of multiple surfaces.
				const FVector SecondMTD = GetPenetrationAdjustment(SweepOutHit);
				const FVector CombinedMTD = Adjustment + SecondMTD;
				if (SecondMTD != Adjustment && !CombinedMTD.IsZero())
				{
					bMoved = MoveUpdatedComponent(CombinedMTD, NewRotationQuat, true, nullptr, ETeleportType::TeleportPhysics);
					UE_LOGF(LogMovement, Verbose, "ResolvePenetration:   sweep by %ls (MTD combo success = %d)", *CombinedMTD.ToString(), bMoved);
				}
			}

			// Still stuck?
			if (!bMoved)
			{
				// Try moving the proposed adjustment plus the attempted move direction. This can sometimes get out of penetrations with multiple objects
				const FVector MoveDelta = ConstrainDirectionToPlane(Hit.TraceEnd - Hit.TraceStart);
				if (!MoveDelta.IsZero())
				{
					bMoved = MoveUpdatedComponent(Adjustment + MoveDelta, NewRotationQuat, true, nullptr, ETeleportType::TeleportPhysics);
					UE_LOGF(LogMovement, Verbose, "ResolvePenetration:   sweep by %ls (adjusted attempt success = %d)", *(Adjustment + MoveDelta).ToString(), bMoved);

					// Finally, try the original move without MTD adjustments, but allowing depenetration along the MTD normal.
					// This was blocked because MOVECOMP_NeverIgnoreBlockingOverlaps was true for the original move to try a better depenetration normal, but we might be running in to other geometry in the attempt.
					// This won't necessarily get us all the way out of penetration, but can in some cases and does make progress in exiting the penetration.
					if (!bMoved && FVector::DotProduct(MoveDelta, Adjustment) > 0.f)
					{
						bMoved = MoveUpdatedComponent(MoveDelta, NewRotationQuat, true, nullptr, ETeleportType::TeleportPhysics);
						UE_LOGF(LogMovement, Verbose, "ResolvePenetration:   sweep by %ls (Original move, attempt success = %d)", *(MoveDelta).ToString(), bMoved);
					}
				}
			}

			return bMoved;
		}
	}

	return false;
}


FVector UMovementComponent::ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit) const
{
	if (!bConstrainToPlane)
	{
		return FVector::VectorPlaneProject(Delta, Normal) * Time;
	}
	else
	{
		const FVector ProjectedNormal = ConstrainNormalToPlane(Normal);
		return FVector::VectorPlaneProject(Delta, ProjectedNormal) * Time;
	}
}


float UMovementComponent::SlideAlongSurface(const FVector& Delta, float Time, const FVector& Normal, FHitResult& Hit, bool bHandleImpact)
{
	if (!Hit.bBlockingHit)
	{
		return 0.f;
	}

	float PercentTimeApplied = 0.f;
	const FVector OldHitNormal = Normal;

	FVector SlideDelta = ComputeSlideVector(Delta, Time, Normal, Hit);

	if ((SlideDelta | Delta) > 0.f)
	{
		const FQuat Rotation = UpdatedComponent->GetComponentQuat();
		SafeMoveUpdatedComponent(SlideDelta, Rotation, true, Hit);

		const float FirstHitPercent = Hit.Time;
		PercentTimeApplied = FirstHitPercent;
		if (Hit.IsValidBlockingHit())
		{
			// Notify first impact
			if (bHandleImpact)
			{
				HandleImpact(Hit, FirstHitPercent * Time, SlideDelta);
			}

			// Compute new slide normal when hitting multiple surfaces.
			TwoWallAdjust(SlideDelta, Hit, OldHitNormal);

			// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
			if (!SlideDelta.IsNearlyZero(1e-3f) && (SlideDelta | Delta) > 0.f)
			{
				// Perform second move
				SafeMoveUpdatedComponent(SlideDelta, Rotation, true, Hit);
				const float SecondHitPercent = Hit.Time * (1.f - FirstHitPercent);
				PercentTimeApplied += SecondHitPercent;

				// Notify second impact
				if (bHandleImpact && Hit.bBlockingHit)
				{
					HandleImpact(Hit, SecondHitPercent * Time, SlideDelta);
				}
			}
		}

		return FMath::Clamp(PercentTimeApplied, 0.f, 1.f);
	}

	return 0.f;
}


void UMovementComponent::TwoWallAdjust(FVector& OutDelta, const FHitResult& Hit, const FVector& OldHitNormal) const
{
	FVector Delta = OutDelta;
	const FVector HitNormal = Hit.Normal;

	if ((OldHitNormal | HitNormal) <= 0.f) //90 or less corner, so use cross product for direction
	{
		const FVector DesiredDir = Delta;
		FVector NewDir = (HitNormal ^ OldHitNormal);
		NewDir = NewDir.GetSafeNormal();
		Delta = (Delta | NewDir) * (1.f - Hit.Time) * NewDir;
		if ((DesiredDir | Delta) < 0.f)
		{
			Delta = -1.f * Delta;
		}
	}
	else //adjust to new wall
	{
		const FVector DesiredDir = Delta;
		Delta = ComputeSlideVector(Delta, 1.f - Hit.Time, HitNormal, Hit);
		if ((Delta | DesiredDir) <= 0.f)
		{
			Delta = FVector::ZeroVector;
		}
		else if ( FMath::Abs((HitNormal | OldHitNormal) - 1.f) < UE_KINDA_SMALL_NUMBER )
		{
			// we hit the same wall again even after adjusting to move along it the first time
			// nudge away from it (this can happen due to precision issues)
			Delta += HitNormal * 0.01f;
		}
	}

	OutDelta = Delta;
}

void UMovementComponent::AddRadialForce(const FVector& Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff)
{
	// Default implementation does nothing
}

void UMovementComponent::AddRadialImpulse(const FVector& Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bVelChange)
{
	// Default implementation does nothing
}

#undef LOCTEXT_NAMESPACE

