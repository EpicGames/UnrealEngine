// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/NavMovementComponent.h"
#include "AI/NavigationSystemBase.h"
#include "Components/CapsuleComponent.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavMovementComponent)

//----------------------------------------------------------------------//
// UMovementComponent
//----------------------------------------------------------------------//
UNavMovementComponent::UNavMovementComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, FixedPathBrakingDistance_DEPRECATED(0.0f)
	, bUpdateNavAgentWithOwnersCollision_DEPRECATED(true)
	, bUseAccelerationForPaths_DEPRECATED(false)
	, bUseFixedBrakingDistanceForPaths_DEPRECATED(false)
	, bStopMovementAbortPaths_DEPRECATED(true)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	bComponentShouldUpdatePhysicsVolume = true;
}

FBasedPosition UNavMovementComponent::GetActorFeetLocationBased() const
{
	return FBasedPosition(NULL, GetActorFeetLocation());
}

void UNavMovementComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.IsPersistent())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// If this is an old version of the component we want to load values from the deprecated ones
		if (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::NavMovementComponentMovingPropertiesToStruct)
		{
			NavMovementProperties.FixedPathBrakingDistance = FixedPathBrakingDistance_DEPRECATED;
			NavMovementProperties.bUpdateNavAgentWithOwnersCollision = bUpdateNavAgentWithOwnersCollision_DEPRECATED;
			NavMovementProperties.bUseAccelerationForPaths = bUseAccelerationForPaths_DEPRECATED;
			NavMovementProperties.bUseFixedBrakingDistanceForPaths = bUseFixedBrakingDistanceForPaths_DEPRECATED;
			NavMovementProperties.bStopMovementAbortPaths = bStopMovementAbortPaths_DEPRECATED;
		}
		// Deprecated properties are not saved, which can mean data loss for derived blueprints when their parent is resaved, so lets keep them up to date for now
		else
		{
			FixedPathBrakingDistance_DEPRECATED = NavMovementProperties.FixedPathBrakingDistance;
			bUpdateNavAgentWithOwnersCollision_DEPRECATED = NavMovementProperties.bUpdateNavAgentWithOwnersCollision;
			bUseAccelerationForPaths_DEPRECATED = NavMovementProperties.bUseAccelerationForPaths;
			bUseFixedBrakingDistanceForPaths_DEPRECATED = NavMovementProperties.bUseFixedBrakingDistanceForPaths;
			bStopMovementAbortPaths_DEPRECATED = NavMovementProperties.bStopMovementAbortPaths;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITORONLY_DATA	
}

#if WITH_EDITOR

void UNavMovementComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Deprecated properties are not saved, which can mean data loss for derived blueprints when their parent is resaved, so lets keep them up to date for now
	if (PropertyChangedEvent.MemberProperty &&
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNavMovementComponent, NavMovementProperties))
	{
		if (PropertyChangedEvent.Property)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FNavMovementProperties, FixedPathBrakingDistance))
			{
				FixedPathBrakingDistance_DEPRECATED = NavMovementProperties.FixedPathBrakingDistance;
			}

			if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FNavMovementProperties, bUpdateNavAgentWithOwnersCollision))
			{
				bUpdateNavAgentWithOwnersCollision_DEPRECATED = NavMovementProperties.bUpdateNavAgentWithOwnersCollision;
			}

			if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FNavMovementProperties, bUseAccelerationForPaths))
			{
				bUseAccelerationForPaths_DEPRECATED = NavMovementProperties.bUseAccelerationForPaths;
			}

			if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FNavMovementProperties, bUseFixedBrakingDistanceForPaths))
			{
				bUseFixedBrakingDistanceForPaths_DEPRECATED = NavMovementProperties.bUseFixedBrakingDistanceForPaths;
			}

			if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FNavMovementProperties, bStopMovementAbortPaths))
			{
				bStopMovementAbortPaths_DEPRECATED = NavMovementProperties.bStopMovementAbortPaths;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

#endif // WITH_EDITOR

void UNavMovementComponent::UpdateNavAgent(const UObject& ObjectToUpdateFrom)
{
	if (!ShouldUpdateNavAgentWithOwnersCollision())
	{
		return;
	}

	// initialize properties from navigation system
	NavAgentProps.NavWalkingSearchHeightScale = FNavigationSystem::GetDefaultSupportedAgent().NavWalkingSearchHeightScale;
	
	if (const UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(&ObjectToUpdateFrom))
	{
		NavAgentProps.AgentRadius = CapsuleComponent->GetScaledCapsuleRadius();
		NavAgentProps.AgentHeight = CapsuleComponent->GetScaledCapsuleHalfHeight() * 2.f;
	}
	else if (const AActor* ObjectAsActor = Cast<AActor>(&ObjectToUpdateFrom))
	{
		ensureMsgf(&ObjectToUpdateFrom == GetOwner(), TEXT("Object passed to UpdateNavAgent should be the owner actor of the Nav Movement Component"));
		// Can't call GetSimpleCollisionCylinder(), because no components will be registered.
		float BoundRadius, BoundHalfHeight;	
		ObjectAsActor->GetSimpleCollisionCylinder(BoundRadius, BoundHalfHeight);
		NavAgentProps.AgentRadius = BoundRadius;
		NavAgentProps.AgentHeight = BoundHalfHeight * 2.f;
	}
}

void UNavMovementComponent::RequestDirectMove(const FVector& MoveVelocity, bool bForceMaxSpeed)
{
	Velocity = MoveVelocity;
}

void UNavMovementComponent::RequestPathMove(const FVector& MoveInput)
{
	// empty in base class, requires at least PawnMovementComponent for input related operations
}

bool UNavMovementComponent::CanStopPathFollowing() const
{
	return true;
}

void UNavMovementComponent::ClearFixedBrakingDistance()
{
	NavMovementProperties.bUseFixedBrakingDistanceForPaths = false;
}

void UNavMovementComponent::GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const
{
	GetOwner()->GetSimpleCollisionCylinder(CollisionRadius, CollisionHalfHeight);
}

FVector UNavMovementComponent::GetSimpleCollisionCylinderExtent() const
{
	return GetOwner()->GetSimpleCollisionCylinderExtent();
}

FVector UNavMovementComponent::GetForwardVector() const
{
	return GetOwner()->GetActorForwardVector();
}

void UNavMovementComponent::SetUpdateNavAgentWithOwnersCollisions(bool bUpdateWithOwner)
{
	NavMovementProperties.bUpdateNavAgentWithOwnersCollision = bUpdateWithOwner;
}
