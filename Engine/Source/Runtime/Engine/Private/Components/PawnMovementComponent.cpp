// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PawnMovementComponent.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "PhysicsEngine/BodyInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PawnMovementComponent)

DEFINE_LOG_CATEGORY_STATIC(LogPawnMovementComponent, Log, All);


//----------------------------------------------------------------------//
// UPawnMovementComponent
//----------------------------------------------------------------------//
void UPawnMovementComponent::SetUpdatedComponent(USceneComponent* NewUpdatedComponent)
{
	if (NewUpdatedComponent)
	{
		if (!ensureMsgf(Cast<APawn>(NewUpdatedComponent->GetOwner()), TEXT("%s must update a component owned by a Pawn"), *GetName()))
		{
			return;
		}
	}

	Super::SetUpdatedComponent(NewUpdatedComponent);

	PawnOwner = UpdatedComponent ? CastChecked<APawn>(UpdatedComponent->GetOwner()) : NULL;
}

void UPawnMovementComponent::Serialize(FArchive& Ar)
{
	APawn* CurrentPawnOwner = PawnOwner;
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		// This was marked Transient so it won't be saved out, but we need still to reject old saved values.
		PawnOwner = CurrentPawnOwner;
	}
}

APawn* UPawnMovementComponent::GetPawnOwner() const
{
	return PawnOwner;
}

bool UPawnMovementComponent::IsMoveInputIgnored() const
{
	if (UpdatedComponent)
	{
		if (PawnOwner)
		{
			return PawnOwner->IsMoveInputIgnored();
		}
	}

	// No UpdatedComponent or Pawn, no movement.
	return true;
}

void UPawnMovementComponent::AddInputVector(FVector WorldAccel, bool bForce /*=false*/)
{
	if (PawnOwner)
	{
		PawnOwner->Internal_AddMovementInput(WorldAccel, bForce);
	}
}

FVector UPawnMovementComponent::GetPendingInputVector() const
{
	return PawnOwner ? PawnOwner->Internal_GetPendingMovementInputVector() : FVector::ZeroVector;
}

FVector UPawnMovementComponent::GetLastInputVector() const
{
	return PawnOwner ? PawnOwner->Internal_GetLastMovementInputVector() : FVector::ZeroVector;
}

FVector UPawnMovementComponent::ConsumeInputVector()
{
	return PawnOwner ? PawnOwner->Internal_ConsumeMovementInputVector() : FVector::ZeroVector;
}

void UPawnMovementComponent::RequestPathMove(const FVector& MoveInput)
{
	if (PawnOwner)
	{
		PawnOwner->Internal_AddMovementInput(MoveInput);
	}
}

void UPawnMovementComponent::OnTeleported()
{
	if (PawnOwner && PawnOwner->IsNetMode(NM_Client) && PawnOwner->IsLocallyControlled())
	{
		MarkForClientCameraUpdate();
	}
}

AController* UPawnMovementComponent::GetController() const
{
	if (PawnOwner)
	{
		return PawnOwner->GetController();
	}

	return nullptr;
}

void UPawnMovementComponent::MarkForClientCameraUpdate()
{

	if (APlayerController* PlayerController = Cast<APlayerController>(GetController()))
	{
		APlayerCameraManager* PlayerCameraManager = PlayerController->PlayerCameraManager;
		if (PlayerCameraManager != nullptr && PlayerCameraManager->bUseClientSideCameraUpdates)
		{
			PlayerCameraManager->bShouldSendClientSideCameraUpdate = true;
		}
	}
}

void UPawnMovementComponent::ApplyAsyncPhysicsStateAction(const UPrimitiveComponent* ActionComponent, const FName& BoneName, 
	const EPhysicsStateAction ActionType, const FVector& ActionDatas, const FVector& ActionPosition)
{
	if (APawn* MyPawn = Cast<APawn>(GetOwner()))
	{
		if (APlayerController* PlayerController = Cast<APlayerController>(MyPawn->GetController()))
		{
			if (PlayerController->IsLocalController() && !PlayerController->IsNetMode(NM_DedicatedServer))
			{
				FAsyncPhysicsTimestamp TimeStamp = PlayerController->GetPhysicsTimestamp();
				ExecuteAsyncPhysicsStateAction(ActionComponent, BoneName, TimeStamp, ActionType, ActionDatas, ActionPosition);

				if (!PlayerController->IsNetMode(NM_Standalone))
				{
					if (!PlayerController->IsNetMode(NM_ListenServer))
					{
						ServerAsyncPhysicsStateAction(ActionComponent, BoneName, TimeStamp, ActionType, ActionDatas, ActionPosition);
					}
					else
					{
						MulticastAsyncPhysicsStateAction(ActionComponent, BoneName, TimeStamp, ActionType, ActionDatas, ActionPosition);
					}
				}
			}
		}
	}
}

void UPawnMovementComponent::ServerAsyncPhysicsStateAction_Implementation(const UPrimitiveComponent* ActionComponent, const FName BoneName, const FAsyncPhysicsTimestamp Timestamp,
	const EPhysicsStateAction ActionType, const FVector ActionDatas, const FVector ActionPosition)
{
	if (ActionComponent)
	{
		ExecuteAsyncPhysicsStateAction(ActionComponent, BoneName, Timestamp, ActionType, ActionDatas, ActionPosition);
		MulticastAsyncPhysicsStateAction(ActionComponent, BoneName, Timestamp, ActionType, ActionDatas, ActionPosition);
	}
}

void UPawnMovementComponent::MulticastAsyncPhysicsStateAction_Implementation(const UPrimitiveComponent* ActionComponent, const FName BoneName, const FAsyncPhysicsTimestamp Timestamp,
	const EPhysicsStateAction ActionType, const FVector ActionDatas, const FVector ActionPosition)
{
	if (!ActionComponent)
	{
		return;
	}

	if (UWorld* World = GetWorld())
	{
		if (APlayerController* PlayerController = World->GetFirstPlayerController())
		{
			if (GetController() == nullptr)
			{
				FAsyncPhysicsTimestamp LocalTimeStamp = Timestamp;
				LocalTimeStamp.LocalFrame = LocalTimeStamp.ServerFrame - PlayerController->GetNetworkPhysicsTickOffset();

				ExecuteAsyncPhysicsStateAction(ActionComponent, BoneName, LocalTimeStamp, ActionType, ActionDatas, ActionPosition);
			}
		}
	}
}

void UPawnMovementComponent::ExecuteAsyncPhysicsStateAction(const UPrimitiveComponent* ActionComponent, const FName& BoneName, const FAsyncPhysicsTimestamp& Timestamp,
	const EPhysicsStateAction ActionType, const FVector& ActionDatas, const FVector& ActionPosition)
{
	if (!ActionComponent)
	{
		return;
	}

	if (FBodyInstance* BI = ActionComponent->GetBodyInstance(BoneName))
	{
		APawn* LocalPawn = Cast<APawn>(GetOwner());
		APlayerController* PlayerController = (LocalPawn && LocalPawn->GetController()) ?
			Cast<APlayerController>(LocalPawn->GetController()) : GetWorld()->GetFirstPlayerController();

		switch (ActionType)
		{
			case EPhysicsStateAction::AddImpulseAtPosition:
				BI->AddImpulseAtPosition(ActionDatas, ActionPosition, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddForceAtPosition:
				BI->AddForceAtPosition(ActionDatas, ActionPosition, true, false, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddVelocityAtPosition:
				BI->AddVelocityChangeImpulseAtLocation(ActionDatas, ActionPosition, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddLinearImpulse:
				BI->AddImpulse(ActionDatas, false, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddForce:
				BI->AddForce(ActionDatas, true, false, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddAcceleration:
				BI->AddForce(ActionDatas, true, true, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddLinearVelocity:
				BI->AddImpulse(ActionDatas, true, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddAngularImpulse:
				BI->AddAngularImpulseInRadians(ActionDatas, false, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddTorque:
				BI->AddTorqueInRadians(ActionDatas, true, false, Timestamp, PlayerController);
				break;
			case EPhysicsStateAction::AddAngularVelocity:
				BI->AddAngularImpulseInRadians(ActionDatas, true, Timestamp, PlayerController);
				break;

		}
	}
}
