// Copyright Epic Games, Inc. All Rights Reserved.


#include "Components/PawnNoiseEmitterComponent.h"
#include "GameFramework/Pawn.h"
#include "Engine/World.h"
#include "GameFramework/Controller.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PawnNoiseEmitterComponent)

UPawnNoiseEmitterComponent::UPawnNoiseEmitterComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NoiseLifetime = 1.0f;
	PrimaryComponentTick.bCanEverTick = false;
	bAIPerceptionSystemCompatibilityMode = true;
}


void UPawnNoiseEmitterComponent::MakeNoise(AActor* NoiseMaker, float Loudness, const FVector& NoiseLocation)
{
	// @hack! this won't be needed once UPawnNoiseEmitterComponent gets moved to AIModule
	// there's no other way to easily and efficiently prevent infinite recursion when 
	// bAIPerceptionSystemCompatibilityMode == true and UAISense_Hearing is not being used (yet)
	static bool bMakeNoiseLockHack = false;
	if (bMakeNoiseLockHack)
	{
		return;
	}

	if (!NoiseMaker || Loudness <= 0.f)
	{
		return;
	}

	// Get the Pawn that owns us, either directly or through a possible Controller owner.
	AActor* Owner = GetOwner();
	APawn* PawnOwner = Cast<APawn>(Owner);
	if (PawnOwner == NULL)
	{
		AController* OwnerController = Cast<AController>(Owner);
		if (IsValid(OwnerController))
		{
			PawnOwner = OwnerController->GetPawn();
		}
	}

	// only emit sounds from pawns with controllers
	if (!PawnOwner || !IsValidChecked(PawnOwner) || !PawnOwner->GetController())
	{
		return;
	}

	// Was this noise made locally by this pawn?
	if ( NoiseMaker == PawnOwner || ((PawnOwner->GetActorLocation() - NoiseLocation).SizeSquared() <= FMath::Square(PawnOwner->GetSimpleCollisionRadius())) )
	{
		// use loudest noise within NoiseLifetime
		// Do not reset volume to zero after time has elapsed; sensors detecting the sound can choose for themselves how long to care about sounds.
		const bool bNoiseExpired = (GetWorld()->GetTimeSeconds() - LastLocalNoiseTime) > NoiseLifetime;
		if (bNoiseExpired || Loudness >= LastLocalNoiseVolume)
		{
			LastLocalNoiseVolume = Loudness;
			LastLocalNoiseTime = GetWorld()->GetTimeSeconds();
		}
	}
	// noise is not local - use loudest in this period
	else
	{
		const bool bNoiseExpired = (GetWorld()->GetTimeSeconds() - LastRemoteNoiseTime) > NoiseLifetime;
		if (bNoiseExpired || Loudness >= LastRemoteNoiseVolume)
		{
			LastRemoteNoiseVolume = Loudness;
			LastRemoteNoisePosition = NoiseLocation;
			LastRemoteNoiseTime = GetWorld()->GetTimeSeconds();
		}
	}

	if (bAIPerceptionSystemCompatibilityMode)
	{
		bMakeNoiseLockHack = true;
		NoiseMaker->MakeNoise(Loudness, PawnOwner, NoiseLocation);
		bMakeNoiseLockHack = false;
	}
}

float UPawnNoiseEmitterComponent::GetLastNoiseVolume(bool bSourceWithinNoiseEmitter) const
{
	if (bSourceWithinNoiseEmitter)
	{
		return LastLocalNoiseVolume;
	}

	return LastRemoteNoiseVolume;
}

float UPawnNoiseEmitterComponent::GetLastNoiseTime(bool bSourceWithinNoiseEmitter) const
{
	if (bSourceWithinNoiseEmitter)
	{
		return LastLocalNoiseTime;
	}

	return LastRemoteNoiseTime;
}

