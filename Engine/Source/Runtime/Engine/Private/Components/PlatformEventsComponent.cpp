// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PlatformEventsComponent.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlatformEventsComponent)


UPlatformEventsComponent::UPlatformEventsComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ }


bool UPlatformEventsComponent::IsInLaptopMode()
{
	return (FPlatformMisc::GetConvertibleLaptopMode() == EConvertibleLaptopMode::Laptop);
}


bool UPlatformEventsComponent::IsInTabletMode()
{
	return (FPlatformMisc::GetConvertibleLaptopMode() == EConvertibleLaptopMode::Tablet);
}


bool UPlatformEventsComponent::SupportsConvertibleLaptops()
{
	return (FPlatformMisc::GetConvertibleLaptopMode() != EConvertibleLaptopMode::NotSupported);
}


void UPlatformEventsComponent::OnRegister()
{
	Super::OnRegister();

	FCoreDelegates::PlatformChangedLaptopMode.AddUObject(this, &UPlatformEventsComponent::HandlePlatformChangedLaptopMode);
}


void UPlatformEventsComponent::OnUnregister()
{
	Super::OnUnregister();
	
 	FCoreDelegates::PlatformChangedLaptopMode.RemoveAll(this);
}


void UPlatformEventsComponent::HandlePlatformChangedLaptopMode(EConvertibleLaptopMode NewMode)
{
	if (NewMode == EConvertibleLaptopMode::Laptop)
	{
		PlatformChangedToLaptopModeDelegate.Broadcast();
	}
	else if (NewMode == EConvertibleLaptopMode::Tablet)
	{
		PlatformChangedToTabletModeDelegate.Broadcast();
	}
}

