// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PointLightComponent.h"
#include "RenderUtils.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"
#include "PointLightSceneProxy.h"
#include "UObject/UnrealType.h"
#include "SceneInterface.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PointLightComponent)

UPointLightComponent::UPointLightComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UTexture2D> StaticTexture(TEXT("/Engine/EditorResources/LightIcons/S_LightPoint"));
		static ConstructorHelpers::FObjectFinder<UTexture2D> DynamicTexture(TEXT("/Engine/EditorResources/LightIcons/S_LightPointMove"));

		StaticEditorTexture = StaticTexture.Object;
		StaticEditorTextureScale = 0.5f;
		DynamicEditorTexture = DynamicTexture.Object;
		DynamicEditorTextureScale = 0.5f;
	}
#endif

	LightFalloffExponent = 8.0f;
	SourceRadius = 0.0f;
	SoftSourceRadius = 0.0f;
	SourceLength = 0.0f;
	bUseInverseSquaredFalloff = true;
}

static bool IsPointLightSupported(const UPointLightComponent* InLight)
{
	if (GMaxRHIFeatureLevel <= ERHIFeatureLevel::ES3_1 && InLight->IsMovable())
	{
		if (!IsMobileDeferredShadingEnabled(GMaxRHIShaderPlatform))
		{
			// if project does not support dynamic point lights on mobile do not add them to the renderer 
			return MobileForwardEnableLocalLights(GMaxRHIShaderPlatform);
		}
	}
	return true;
}

FLightSceneProxy* UPointLightComponent::CreateSceneProxy() const
{
	if (IsPointLightSupported(this) && !IsPSOPrecaching())
	{
		return new FPointLightSceneProxy(this);
	}
	return nullptr;
}

void UPointLightComponent::PrecachePSOs()
{
	if (IsPointLightSupported(this))
	{
		Super::PrecachePSOs();
	}
}

void UPointLightComponent::SetUseInverseSquaredFalloff(bool bNewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& bUseInverseSquaredFalloff != bNewValue)
	{
		bUseInverseSquaredFalloff = bNewValue;
		MarkRenderStateDirty();
	}
}

void UPointLightComponent::SetLightFalloffExponent(float NewLightFalloffExponent)
{
	if (AreDynamicDataChangesAllowed()
		&& NewLightFalloffExponent != LightFalloffExponent)
	{
		LightFalloffExponent = NewLightFalloffExponent;
		MarkRenderStateDirty();
	}
}

void UPointLightComponent::SetInverseExposureBlend(float NewInverseExposureBlend)
{
	if (AreDynamicDataChangesAllowed()
		&& NewInverseExposureBlend != InverseExposureBlend)
	{
		InverseExposureBlend = NewInverseExposureBlend;
		MarkRenderStateDirty();
	}
}

void UPointLightComponent::SetSourceRadius(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& SourceRadius != NewValue)
	{
		SourceRadius = NewValue;
		MarkRenderStateDirty();
	}
}

void UPointLightComponent::SetSoftSourceRadius(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& SoftSourceRadius != NewValue)
	{
		SoftSourceRadius = NewValue;
		MarkRenderStateDirty();
	}
}

void UPointLightComponent::SetSourceLength(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& SourceLength != NewValue)
	{
		SourceLength = NewValue;
		MarkRenderStateDirty();
	}
}

float UPointLightComponent::ComputeLightBrightness() const
{
	float LightBrightness = Super::ComputeLightBrightness();

	if (bUseInverseSquaredFalloff)
	{
		if (IntensityUnits == ELightUnits::Candelas)
		{
			LightBrightness *= (100.f * 100.f); // Conversion from cm2 to m2
		}
		else if (IntensityUnits == ELightUnits::Nits)
		{
			// Capsule area (sphere+cylinder)
			float AreaInCm2 = (4 * UE_PI) * SourceRadius * (SourceRadius + 0.5f * SourceLength);
			LightBrightness *= AreaInCm2;
		}
		else if (IntensityUnits == ELightUnits::Lumens)
		{
			LightBrightness *= (100.f * 100.f / 4 / UE_PI); // Conversion from cm2 to m2 and 4PI from the sphere area in the 1/r2 attenuation
		}
		else if (IntensityUnits == ELightUnits::EV)
		{
			LightBrightness = EV100ToLuminance(LightBrightness) * (100.f * 100.f);
		}
		else
		{
			LightBrightness *= 16; // Legacy scale of 16
		}
	}
	return LightBrightness;
}

#if WITH_EDITOR
void UPointLightComponent::SetLightBrightness(float InBrightness)
{
	if (bUseInverseSquaredFalloff)
	{
		if (IntensityUnits == ELightUnits::Candelas)
		{
			Super::SetLightBrightness(InBrightness / (100.f * 100.f)); // Conversion from cm2 to m2
		}
		else if (IntensityUnits == ELightUnits::Nits)
		{
			// Capsule area (sphere+cylinder)
			float AreaInCm2 = (4 * UE_PI) * SourceRadius * (SourceRadius + 0.5f * SourceLength);
			Super::SetLightBrightness(InBrightness / AreaInCm2);
		}
		else if (IntensityUnits == ELightUnits::Lumens)
		{
			Super::SetLightBrightness(InBrightness / (100.f * 100.f / 4 / UE_PI)); // Conversion from cm2 to m2 and 4PI from the sphere area in the 1/r2 attenuation
		}
		else if (IntensityUnits == ELightUnits::EV)
		{
			Super::SetLightBrightness(LuminanceToEV100(InBrightness / (100.f * 100.f)));
		}
		else
		{
			Super::SetLightBrightness(InBrightness / 16); // Legacy scale of 16
		}
	}
	else
	{
		Super::SetLightBrightness(InBrightness);
	}
}
#endif // WITH_EDITOR

/**
* @return ELightComponentType for the light component class 
*/
ELightComponentType UPointLightComponent::GetLightType() const
{
	return LightType_Point;
}

float UPointLightComponent::GetUniformPenumbraSize() const
{
	if (LightmassSettings.bUseAreaShadowsForStationaryLight)
	{
		// Interpret distance as shadow factor directly
		return 1.0f;
	}
	else
	{
		// Heuristic to derive uniform penumbra size from light source radius
		return FMath::Clamp(SourceRadius == 0 ? .05f : SourceRadius * .005f, .0001f, 1.0f);
	}
}

void UPointLightComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.UEVer() < VER_UE4_INVERSE_SQUARED_LIGHTS_DEFAULT)
	{
		bUseInverseSquaredFalloff = InverseSquaredFalloff_DEPRECATED;
	}
	// Reorient old light tubes that didn't use an IES profile
	else if(Ar.UEVer() < VER_UE4_POINTLIGHT_SOURCE_ORIENTATION && SourceLength > UE_KINDA_SMALL_NUMBER && IESTexture == nullptr)
	{
		AddLocalRotation( FRotator(-90.f, 0.f, 0.f) );
	}

	if (Ar.IsLoading() && !bUseInverseSquaredFalloff)
	{
		IntensityUnits = ELightUnits::Unitless;
	}
}

#if WITH_EDITOR

bool UPointLightComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, LightFalloffExponent))
		{
			return !bUseInverseSquaredFalloff;
		}
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ULocalLightComponent, IntensityUnits))
		{
			return bUseInverseSquaredFalloff;
		}
	}

	return Super::CanEditChange(InProperty);
}

/**
 * Called after property has changed via e.g. property window or set command.
 *
 * @param	PropertyThatChanged	FProperty that has been changed, NULL if unknown
 */
void UPointLightComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Make sure exponent is > 0.
	LightFalloffExponent = FMath::Max( (float)UE_KINDA_SMALL_NUMBER, LightFalloffExponent );
	SourceRadius = FMath::Max(0.0f, SourceRadius);
	SoftSourceRadius = FMath::Max(0.0f, SoftSourceRadius);
	SourceLength = FMath::Max(0.0f, SourceLength);
	InverseExposureBlend = FMath::Clamp(InverseExposureBlend, 0.0f, 1.0f);

	if (!bUseInverseSquaredFalloff)
	{
		IntensityUnits = ELightUnits::Unitless;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UPointLightComponent::BuildSceneProxyDesc(const UPointLightComponent& InLightComponent, FPointLightSceneProxyDesc& OutProxyDesc)
{
	ULocalLightComponent::BuildSceneProxyDesc(InLightComponent, OutProxyDesc);

	OutProxyDesc.LightFalloffExponent = InLightComponent.LightFalloffExponent;
	OutProxyDesc.SourceRadius = InLightComponent.SourceRadius;
	OutProxyDesc.SoftSourceRadius = InLightComponent.SoftSourceRadius;
	OutProxyDesc.SourceLength = InLightComponent.SourceLength;
	OutProxyDesc.bUseInverseSquaredFalloff = InLightComponent.bUseInverseSquaredFalloff;
}