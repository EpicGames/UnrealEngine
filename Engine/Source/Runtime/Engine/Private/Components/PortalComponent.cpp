// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	
=============================================================================*/

#include "Async/TaskGraphInterfaces.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/BillboardComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Texture2D.h"
#include "Lightmass/LightmassPortal.h"
#include "Components/LightmassPortalComponent.h"
#include "Components/BoxComponent.h"

ALightmassPortal::ALightmassPortal(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PortalComponent = CreateDefaultSubobject<ULightmassPortalComponent>(TEXT("PortalComponent"));
	PortalComponent->SetRelativeScale3D_Direct(FVector(10, 100, 100));
	RootComponent = PortalComponent;
	UBoxComponent* DrawInfluenceBox = CreateDefaultSubobject<UBoxComponent>(TEXT("DrawBox0"));
	DrawInfluenceBox->SetupAttachment(GetPortalComponent());
	DrawInfluenceBox->bUseEditorCompositing = true;
	DrawInfluenceBox->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	DrawInfluenceBox->InitBoxExtent(FVector(1, 1, 1));
	PortalComponent->PreviewBox = DrawInfluenceBox;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (!IsRunningCommandlet() && (SpriteComponent != nullptr))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> DecalTexture;
			FConstructorStatics()
				: DecalTexture(TEXT("/Engine/EditorResources/S_PortalActorIcon2"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.DecalTexture.Get();
		SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->bHiddenInGame = true;
		SpriteComponent->SetUsingAbsoluteScale(true);
		SpriteComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		SpriteComponent->bIsScreenSizeScaled = true;
		SpriteComponent->SetupAttachment(PortalComponent);
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void ALightmassPortal::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	PortalComponent->GetRelativeScale3D_DirectMutable().X = 10;
	PortalComponent->MarkRenderStateDirty();
}
#endif // WITH_EDITOR

ULightmassPortalComponent::ULightmassPortalComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Mobility = EComponentMobility::Static;
}

void ULightmassPortalComponent::UpdatePreviewShape()
{
	if (PreviewBox)
	{
		PreviewBox->InitBoxExtent(FVector(1));
	}
}

void ULightmassPortalComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	UpdatePreviewShape();
}

void ULightmassPortalComponent::SendRenderTransform_Concurrent()
{	
	UpdatePreviewShape();

	Super::SendRenderTransform_Concurrent();
}
