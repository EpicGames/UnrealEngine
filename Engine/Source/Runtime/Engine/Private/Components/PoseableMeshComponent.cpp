// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PoseableMeshComponent.cpp: UPoseableMeshComponent methods.
=============================================================================*/

#include "Components/PoseableMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimInstance.h"
#include "Engine/SkinnedAsset.h"
#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseableMeshComponent)

UPoseableMeshComponent::UPoseableMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bNeedsRefreshTransform (false)
{

}

bool UPoseableMeshComponent::AllocateTransformData()
{
	// Allocate transforms if not present.
	if ( Super::AllocateTransformData() )
	{
		if(BoneSpaceTransforms.Num() != GetSkinnedAsset()->GetRefSkeleton().GetNum() )
		{
			BoneSpaceTransforms = GetSkinnedAsset()->GetRefSkeleton().GetRefBonePose();

			TArray<FBoneIndexType> RequiredBoneIndexArray;
			RequiredBoneIndexArray.AddUninitialized(BoneSpaceTransforms.Num());
			for(int32 BoneIndex = 0; BoneIndex < BoneSpaceTransforms.Num(); ++BoneIndex)
			{
				RequiredBoneIndexArray[BoneIndex] = BoneIndex;
			}

			RequiredBones.InitializeTo(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *GetSkinnedAsset());
		}

		FillComponentSpaceTransforms();
		FinalizeBoneTransform();

		return true;
	}

	BoneSpaceTransforms.Empty();

	return false;
}

void UPoseableMeshComponent::RefreshBoneTransforms(FActorComponentTickFunction* TickFunction)
{
	SCOPE_CYCLE_COUNTER(STAT_RefreshBoneTransforms);

	// Can't do anything without a SkinnedAsset
	if( !GetSkinnedAsset())
	{
		return;
	}

	// Do nothing more if no bones in skeleton.
	if( GetNumComponentSpaceTransforms() == 0 )
	{
		return;
	}

	// We need the mesh space bone transforms now for renderer to get delta from ref pose:
	FillComponentSpaceTransforms();
	FinalizeBoneTransform();

	UpdateChildTransforms();
	UpdateBounds();
	MarkRenderTransformDirty();
	MarkRenderDynamicDataDirty();

	bNeedsRefreshTransform = false;
}

void UPoseableMeshComponent::FillComponentSpaceTransforms()
{
	ANIM_MT_SCOPE_CYCLE_COUNTER(FillComponentSpaceTransforms, IsRunningParallelEvaluation());

	if( !GetSkinnedAsset())
	{
		return;
	}

	// If skeleton has changed then attempt to reallocate internal transform data
	if (BoneSpaceTransforms.Num() != GetSkinnedAsset()->GetRefSkeleton().GetNum())
	{
		verify(AllocateTransformData());
	}

	// right now all this does is to convert to SpaceBases
	check( GetSkinnedAsset()->GetRefSkeleton().GetNum() == BoneSpaceTransforms.Num() );
	check( GetSkinnedAsset()->GetRefSkeleton().GetNum() == GetNumComponentSpaceTransforms());
	check( GetSkinnedAsset()->GetRefSkeleton().GetNum() == GetBoneVisibilityStates().Num() );

	const int32 NumBones = BoneSpaceTransforms.Num();

#if DO_GUARD_SLOW
	/** Keep track of which bones have been processed for fast look up */
	TArray<uint8, TInlineAllocator<256>> BoneProcessed;
	BoneProcessed.AddZeroed(NumBones);
#endif
	// Build in 3 passes.
	FTransform* LocalTransformsData = BoneSpaceTransforms.GetData();
	FTransform* SpaceBasesData = GetEditableComponentSpaceTransforms().GetData();
	
	GetEditableComponentSpaceTransforms()[0] = BoneSpaceTransforms[0];
#if DO_GUARD_SLOW
	BoneProcessed[0] = 1;
#endif

	for(int32 BoneIndex=1; BoneIndex<BoneSpaceTransforms.Num(); BoneIndex++)
	{
		FPlatformMisc::Prefetch(SpaceBasesData + BoneIndex);

#if DO_GUARD_SLOW
		// Mark bone as processed
		BoneProcessed[BoneIndex] = 1;
#endif
		// For all bones below the root, final component-space transform is relative transform * component-space transform of parent.
		const int32 ParentIndex = GetSkinnedAsset()->GetRefSkeleton().GetParentIndex(BoneIndex);
		FPlatformMisc::Prefetch(SpaceBasesData + ParentIndex);

#if DO_GUARD_SLOW
		// Check the precondition that Parents occur before Children in the RequiredBones array.
		checkSlow(BoneProcessed[ParentIndex] == 1);
#endif
		FTransform::Multiply(SpaceBasesData + BoneIndex, LocalTransformsData + BoneIndex, SpaceBasesData + ParentIndex);

		checkSlow(GetEditableComponentSpaceTransforms()[BoneIndex].IsRotationNormalized());
		checkSlow(!GetEditableComponentSpaceTransforms()[BoneIndex].ContainsNaN());
	}
	bNeedToFlipSpaceBaseBuffers = true;
}

void UPoseableMeshComponent::SetBoneTransformByName(FName BoneName, const FTransform& InTransform, EBoneSpaces::Type BoneSpace)
{
	if( !GetSkinnedAsset() || !RequiredBones.IsValid() )
	{
		return;
	}

	check(!LeaderPoseComponent.IsValid()); //Shouldn't call set bone functions when we are using LeaderPoseComponent

	int32 BoneIndex = GetBoneIndex(BoneName);
	if(BoneIndex >=0 && BoneIndex < BoneSpaceTransforms.Num())
	{
		BoneSpaceTransforms[BoneIndex] = InTransform;

		// If we haven't requested local space we need to transform the position passed in
		//if(BoneSpace != EBoneSpaces::LocalSpace)
		{
			if(BoneSpace == EBoneSpaces::WorldSpace)
			{
				BoneSpaceTransforms[BoneIndex].SetToRelativeTransform(GetComponentToWorld());
			}

			int32 ParentIndex = RequiredBones.GetParentBoneIndex(BoneIndex);
			if(ParentIndex >=0)
			{
				FA2CSPose CSPose;
				CSPose.AllocateLocalPoses(RequiredBones, BoneSpaceTransforms);

				BoneSpaceTransforms[BoneIndex].SetToRelativeTransform(CSPose.GetComponentSpaceTransform(ParentIndex));
			}

			MarkRefreshTransformDirty();
		}
	}
}

void UPoseableMeshComponent::SetBoneLocationByName(FName BoneName, FVector InLocation, EBoneSpaces::Type BoneSpace)
{
	FTransform CurrentTransform = GetBoneTransformByName(BoneName, BoneSpace);
	CurrentTransform.SetLocation(InLocation);
	SetBoneTransformByName(BoneName, CurrentTransform, BoneSpace);
}

void UPoseableMeshComponent::SetBoneRotationByName(FName BoneName, FRotator InRotation, EBoneSpaces::Type BoneSpace)
{
	FTransform CurrentTransform = GetBoneTransformByName(BoneName, BoneSpace);
	CurrentTransform.SetRotation(FQuat(InRotation));
	SetBoneTransformByName(BoneName, CurrentTransform, BoneSpace);
}

void UPoseableMeshComponent::SetBoneScaleByName(FName BoneName, FVector InScale3D, EBoneSpaces::Type BoneSpace)
{
	FTransform CurrentTransform = GetBoneTransformByName(BoneName, BoneSpace);
	CurrentTransform.SetScale3D(InScale3D);
	SetBoneTransformByName(BoneName, CurrentTransform, BoneSpace);
}

template<class CompType>
FTransform GetBoneTransformByNameHelper(FName BoneName, EBoneSpaces::Type BoneSpace, FBoneContainer& RequiredBones, CompType* Component)
{
	int32 BoneIndex = Component->GetBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		FString Message = FString::Printf(TEXT("Invalid Bone Name '%s'"), *BoneName.ToString());
		FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
		return FTransform();
	}

	/*if(BoneSpace == EBoneSpaces::LocalSpace)
	{
		return Component->BoneSpaceTransforms[i];
	}*/

	FA2CSPose CSPose;
	CSPose.AllocateLocalPoses(RequiredBones, Component->GetBoneSpaceTransforms());

	if (BoneSpace == EBoneSpaces::ComponentSpace)
	{
		return CSPose.GetComponentSpaceTransform(BoneIndex);
	}
	else
	{
		return CSPose.GetComponentSpaceTransform(BoneIndex) * Component->GetComponentTransform();
	}
}

FTransform UPoseableMeshComponent::GetBoneTransformByName(FName BoneName, EBoneSpaces::Type BoneSpace)
{
	if( !GetSkinnedAsset() || !RequiredBones.IsValid() )
	{
		return FTransform();
	}

	USkinnedMeshComponent* MPCPtr = LeaderPoseComponent.Get();
	if (MPCPtr)
	{
		if (USkeletalMeshComponent* SMC = Cast<USkeletalMeshComponent>(MPCPtr))
		{
			if (UAnimInstance* AnimInstance = SMC->GetAnimInstance())
			{
				return GetBoneTransformByNameHelper(BoneName, BoneSpace, AnimInstance->GetRequiredBones(), SMC);
			}
			FString Message = FString::Printf(TEXT("Cannot return valid bone transform. Leader Pose Component has no anim instance"));
			FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
			return FTransform();
		}
		FString Message = FString::Printf(TEXT("Cannot return valid bone transform. Leader Pose Component is not of type USkeletalMeshComponent"));
		FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
		return FTransform();
	}
	return GetBoneTransformByNameHelper(BoneName, BoneSpace, RequiredBones, this);
}

FVector UPoseableMeshComponent::GetBoneLocationByName(FName BoneName, EBoneSpaces::Type BoneSpace)
{
	FTransform CurrentTransform = GetBoneTransformByName(BoneName, BoneSpace);
	return CurrentTransform.GetLocation();
}

FRotator UPoseableMeshComponent::GetBoneRotationByName(FName BoneName, EBoneSpaces::Type BoneSpace)
{
	FTransform CurrentTransform = GetBoneTransformByName(BoneName, BoneSpace);
	return FRotator(CurrentTransform.GetRotation());
}

FVector UPoseableMeshComponent::GetBoneScaleByName(FName BoneName, EBoneSpaces::Type BoneSpace)
{
	FTransform CurrentTransform = GetBoneTransformByName(BoneName, BoneSpace);
	return CurrentTransform.GetScale3D();
}

void UPoseableMeshComponent::ResetBoneTransformByName(FName BoneName)
{
	if( !GetSkinnedAsset())
	{
		return;
	}

	const int32 BoneIndex = GetBoneIndex(BoneName);
	if( BoneIndex != INDEX_NONE )
	{
		BoneSpaceTransforms[BoneIndex] = GetSkinnedAsset()->GetRefSkeleton().GetRefBonePose()[BoneIndex];
	}
	else
	{
		FString Message = FString::Printf(TEXT("Invalid Bone Name '%s'"), *BoneName.ToString());
		FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
	}
}

void UPoseableMeshComponent::CopyPoseFromSkeletalComponent(USkeletalMeshComponent* InComponentToCopy)
{
	if(RequiredBones.IsValid())
	{
		TArray<FTransform> LocalTransforms = InComponentToCopy->GetBoneSpaceTransforms();
		if (LocalTransforms.IsEmpty()) // during incremental unregistering this might be empty
		{
			return;
		}

		if(this->GetSkinnedAsset() == InComponentToCopy->GetSkinnedAsset()
			&& LocalTransforms.Num() == BoneSpaceTransforms.Num())
		{
			
			Exchange(BoneSpaceTransforms, LocalTransforms);
		}
		else
		{
			// The meshes don't match, search bone-by-bone (slow path)

			// first set the localatoms to ref pose from our current mesh
			BoneSpaceTransforms = GetSkinnedAsset()->GetRefSkeleton().GetRefBonePose();

			// Now overwrite any matching bones
			const int32 NumSourceBones = InComponentToCopy->GetSkinnedAsset()->GetRefSkeleton().GetNum();

			for(int32 SourceBoneIndex = 0 ; SourceBoneIndex < NumSourceBones ; ++SourceBoneIndex)
			{
				const FName SourceBoneName = InComponentToCopy->GetBoneName(SourceBoneIndex);
				const int32 TargetBoneIndex = GetBoneIndex(SourceBoneName);

				if(TargetBoneIndex != INDEX_NONE)
				{
					BoneSpaceTransforms[TargetBoneIndex] = LocalTransforms[SourceBoneIndex];
				}
			}
		}

		MarkRefreshTransformDirty();
	}
}

bool UPoseableMeshComponent::ShouldUpdateTransform(bool bLODHasChanged) const
{
	// we don't always update transform - each function when they changed will update
	return Super::ShouldUpdateTransform(bLODHasChanged) || bNeedsRefreshTransform;
}

void UPoseableMeshComponent::MarkRefreshTransformDirty()
{
	bNeedsRefreshTransform = true;
}
