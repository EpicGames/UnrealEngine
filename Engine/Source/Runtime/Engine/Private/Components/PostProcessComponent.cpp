// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PostProcessComponent.h"
#include "Components/SphereComponent.h"
#include "Engine/World.h"
#include "VolumeBoundsMesh.h"
#include "Templates/PimplPtr.h"

#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Levelset.h"
#include "Chaos/MLLevelset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PostProcessComponent)

UPostProcessComponent::UPostProcessComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bEnabled = true;
	BlendRadius = 100.0f;
	BlendWeight = 1.0f;
	Priority = 0;
	bUnbound = 1;
}

void UPostProcessComponent::OnRegister()
{
	Super::OnRegister();
	GetWorld()->AddPostProcessVolume(this);
}

void UPostProcessComponent::OnUnregister()
{
	Super::OnUnregister();
	GetWorld()->RemovePostProcessVolume(this);
}

void UPostProcessComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		Settings.OnAfterLoad();
#endif
	}
}

TOptional<FBoxSphereBounds> UPostProcessComponent::GetPostProcessBounds() const
{
	if (!bUnbound)
	{
		TArray<Chaos::FPhysicsObjectHandle> VolumeBoundsPhysicsObjects {};
		if (VolumeBoundsMesh && VolumeBoundsMesh->PhysicsObjectsAccessor)
		{
			VolumeBoundsPhysicsObjects = VolumeBoundsMesh->PhysicsObjectsAccessor();
		}

		if (!VolumeBoundsPhysicsObjects.IsEmpty())
		{
			FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(VolumeBoundsPhysicsObjects);
			FBox BoxBounds = Interface->GetWorldBounds(VolumeBoundsPhysicsObjects);
			return FBoxSphereBounds(BoxBounds);
		}
		else if (const UPrimitiveComponent* Primitive = Cast<UPrimitiveComponent>(GetAttachParent()))
		{
			return Primitive->GetBounds();
		}
		else if (const UShapeComponent* Shape = Cast<UShapeComponent>(GetAttachParent()))
		{
			return Shape->GetBounds();
		}
	}
	return NullOpt;
}

void UPostProcessComponent::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (!IsTemplate() && !VolumeGuid.IsValid())
	{
		VolumeGuid = FGuid::NewDeterministicGuid(GetPathName());
	}
#endif
}

// Distance queries require manifold geometry to get meaningful results.
// If a piece of geometry is not 'watertight', you cannot tell what's inside vs outside.
// This function filters out triangle mesh geometry representations, which may be non-manifold.
static bool IsGuaranteedManifoldGeometry(const Chaos::FImplicitObject* Geometry)
{
	if (!Geometry)
	{
		return false;
	}
	else if (const Chaos::FImplicitObjectInstanced* ObjInstanced = Geometry->AsA<Chaos::FImplicitObjectInstanced>())
	{
		return IsGuaranteedManifoldGeometry(ObjInstanced->GetInnerObject().Get());
	}
	else if (const Chaos::FImplicitObjectScaled* ObjScaled = Geometry->AsA<Chaos::FImplicitObjectScaled>())
	{
		return IsGuaranteedManifoldGeometry(ObjScaled->GetInnerObject().Get());
	}
	else if (const Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>* ObjTransformed = Geometry->AsA<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>())
	{
		return IsGuaranteedManifoldGeometry(ObjTransformed->GetGeometry());
	}
	else if (const Chaos::FImplicitObjectUnion* ObjUnion = Geometry->AsA<Chaos::FImplicitObjectUnion>())
	{
		for (const Chaos::FImplicitObject* Element : ObjUnion->GetObjects())
		{
			if (!IsGuaranteedManifoldGeometry(Element))
			{
				return false;
			}
		}
		return true;
	}
	else if (Geometry->IsConvex()
		|| Geometry->IsA<Chaos::FLevelSet>()
		|| Geometry->IsA<Chaos::FMLLevelSet>())
	{
		return true;
	}
	return false;
}

template<typename TVisitor>
void VisitGuaranteedManifoldGeometry(TArray<Chaos::FPhysicsObjectHandle>& PhysicsObjects, TVisitor Visitor)
{
	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(PhysicsObjects);
	Interface->VisitEveryShape(PhysicsObjects, [Visitor](const Chaos::FConstPhysicsObjectHandle Handle, Chaos::TThreadShapeInstance<Chaos::EThreadContext::External>* ShapeInstance) -> bool
	{
		const Chaos::FImplicitObjectRef Geometry = ShapeInstance->GetGeometry();
		if (IsGuaranteedManifoldGeometry(Geometry))
		{
			Visitor(Handle, Geometry);
		}
		return false;
	});
}

static FClosestPhysicsObjectResult FindApproximateDistanceTo(TArray<Chaos::FPhysicsObjectHandle>& PhysicsObjects, FVector WorldLocation, FLockedReadPhysicsObjectExternalInterface& Interface)
{
	FClosestPhysicsObjectResult AggregateResult {};
	VisitGuaranteedManifoldGeometry(PhysicsObjects, [&AggregateResult, &Interface, WorldLocation](const Chaos::FConstPhysicsObjectHandle Object, const Chaos::FImplicitObjectRef Geometry)
	{
		const FTransform WorldTransform = Interface->GetTransform(Object);
		const FVector LocalLocation = WorldTransform.InverseTransformPosition(WorldLocation);

		FClosestPhysicsObjectResult Result;

		Result.PhysicsObject = const_cast<Chaos::FPhysicsObjectHandle>(Object);

		Chaos::FVec3 Normal;
		Result.ClosestDistance = static_cast<double>(Geometry->PhiWithNormal(LocalLocation, Normal));
		Result.ClosestLocation = WorldTransform.TransformPosition(LocalLocation - Result.ClosestDistance * Normal);

		if (!AggregateResult || Result.ClosestDistance < AggregateResult.ClosestDistance)
		{
			AggregateResult = Result;
		}
	});
	return AggregateResult;
}

bool UPostProcessComponent::EncompassesPoint(FVector Point, float SphereRadius/*=0.f*/, float* OutDistanceToPoint)
{
	const TOptional<FBoxSphereBounds> PPBounds = GetPostProcessBounds();
	if (!PPBounds.IsSet()) // global volume encompasses all
	{
		if (OutDistanceToPoint)
		{
			*OutDistanceToPoint = 0;
		}
		return true;
	}

	const bool bCouldBeInBlendingRange = PPBounds->GetBox().ExpandBy(BlendRadius + SphereRadius).IsInsideOrOn(Point);
	if (!bCouldBeInBlendingRange)
	{
		if (OutDistanceToPoint)
		{
			*OutDistanceToPoint = FLT_MAX;
		}
		return false;
	}

	TArray<Chaos::FPhysicsObjectHandle> ActiveVolumeBoundsPhysicsObjects;
	if (VolumeBoundsMesh && VolumeBoundsMesh->PhysicsObjectsAccessor)
	{
		ActiveVolumeBoundsPhysicsObjects = VolumeBoundsMesh->PhysicsObjectsAccessor();
	}
	else if (UPrimitiveComponent* ParentPrimitive = Cast<UPrimitiveComponent>(GetAttachParent()))
	{
		ActiveVolumeBoundsPhysicsObjects = ParentPrimitive->GetAllPhysicsObjects();
	}

	if (!ActiveVolumeBoundsPhysicsObjects.IsEmpty())
	{
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(ActiveVolumeBoundsPhysicsObjects);
		FClosestPhysicsObjectResult QueryResult = FindApproximateDistanceTo(ActiveVolumeBoundsPhysicsObjects, Point, Interface);
		float Distance = FMath::Max(0.0f, static_cast<float>(QueryResult.ClosestDistance));
		if (QueryResult.PhysicsObject == nullptr) // no valid physics object found
		{
			if (OutDistanceToPoint)
			{
				*OutDistanceToPoint = FLT_MAX;
			}
			return false;
		}
		if (OutDistanceToPoint)
		{
			*OutDistanceToPoint = Distance;
		}
		return Distance <= SphereRadius;
	}
	else if (UShapeComponent* ParentShape = Cast<UShapeComponent>(GetAttachParent()))
	{
		float Distance = -1.f;

		FVector ClosestPoint;
		float DistanceSq = -1.f;

		if (ParentShape->GetSquaredDistanceToCollision(Point, DistanceSq, ClosestPoint))
		{
			Distance = FMath::Sqrt(DistanceSq);
		}
		else
		{
			FBoxSphereBounds SphereBounds = ParentShape->CalcBounds(ParentShape->GetComponentTransform());	
			if (ParentShape->IsA<USphereComponent>())
			{
				const FSphere& Sphere = SphereBounds.GetSphere();
				const FVector& Dist = Sphere.Center - Point;
				Distance = FMath::Max(0.0f, Dist.Size() - Sphere.W);
			}
			else // UBox or UCapsule shape (approx).
			{
				Distance = FMath::Sqrt(SphereBounds.GetBox().ComputeSquaredDistanceToPoint(Point));
			}
		}

		if (OutDistanceToPoint)
		{
			*OutDistanceToPoint = Distance;
		}

		return Distance >= 0.f && Distance <= SphereRadius;
	}
	if (OutDistanceToPoint != nullptr)
	{
		*OutDistanceToPoint = 0;
	}
	return true;
}

