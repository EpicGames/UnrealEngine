// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/ProjectileMovementComponent.h"
#include "GameFramework/DamageType.h"
#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/WorldSettings.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProjectileMovementComponent)

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);
DEFINE_LOG_CATEGORY_STATIC(LogProjectileMovement, Log, All);
DEFINE_LOG_CATEGORY_STATIC(LogProjectileMovementInterpolation, Log, All);

const float UProjectileMovementComponent::MIN_TICK_TIME = 1e-6f;

UProjectileMovementComponent::UProjectileMovementComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUpdateOnlyIfRendered = false;
	bInitialVelocityInLocalSpace = true;
	bForceSubStepping = false;
	bSimulationEnabled = true;
	bSweepCollision = true;
	bInterpMovement = false;
	bInterpRotation = false;
	bInterpolationComplete = true;
	bSimulationUseScopedMovement = false;
	bInterpolationUseScopedMovement = true;
	InterpLocationTime = 0.100f;
	InterpRotationTime = 0.050f;
	InterpLocationMaxLagDistance = 300.0f;
	InterpLocationSnapToTargetDistance = 500.0f;
	bThrottleInterpolation = false;
	ThrottleInterpolationThresholdNotRenderedShortTime = 0.20f;
	ThrottleInterpolationThresholdNotRenderedLongTime = 1.0f;
	ThrottleInterpolationFramesSinceInterp = 0;
	ThrottleInterpolationSkipFramesRecent = 1;
	ThrottleInterpolationSkipFramesNotRecent = 2;

	Velocity = FVector(1.f,0.f,0.f);

	ProjectileGravityScale = 1.f;

	Bounciness = 0.6f;
	Friction = 0.2f;
	BounceVelocityStopSimulatingThreshold = 5.f;
	MinFrictionFraction = 0.0f;

	HomingAccelerationMagnitude = 0.f;

	bWantsInitializeComponent = true;
	bComponentShouldUpdatePhysicsVolume = false;

	MaxSimulationTimeStep = 0.05f;
	MaxSimulationIterations = 4;
	BounceAdditionalIterations = 1;

	bBounceAngleAffectsFriction = false;
	bIsSliding = false;
	PreviousHitTime = 1.f;
	PreviousHitNormal = FVector::UpVector;
}

void UProjectileMovementComponent::PostLoad()
{
	Super::PostLoad();

	const FPackageFileVersion LinkerUEVer = GetLinkerUEVersion();

	if (LinkerUEVer < VER_UE4_REFACTOR_PROJECTILE_MOVEMENT)
	{
		// Old code used to treat Bounciness as Friction as well.
		Friction = FMath::Clamp(1.f - Bounciness, 0.f, 1.f);

		// Old projectiles probably don't want to use this behavior by default.
		bInitialVelocityInLocalSpace = false;
	}
}


void UProjectileMovementComponent::InitializeComponent()
{
	Super::InitializeComponent();

	if (Velocity.SizeSquared() > 0.f)
	{
		// InitialSpeed > 0 overrides initial velocity magnitude.
		if (InitialSpeed > 0.f)
		{
			Velocity = Velocity.GetSafeNormal() * InitialSpeed;
		}

		if (bInitialVelocityInLocalSpace)
		{
			SetVelocityInLocalSpace(Velocity);
		}

		if (bRotationFollowsVelocity)
		{
			if (UpdatedComponent)
			{
				FRotator DesiredRotation = Velocity.Rotation();
				if (bRotationRemainsVertical)
				{
					DesiredRotation.Pitch = 0.0f;
					DesiredRotation.Yaw = FRotator::NormalizeAxis(DesiredRotation.Yaw);
					DesiredRotation.Roll = 0.0f;
				}

				UpdatedComponent->SetWorldRotation(DesiredRotation);
			}
		}

		UpdateComponentVelocity();
		
		if (UpdatedPrimitive && UpdatedPrimitive->IsSimulatingPhysics())
		{
			UpdatedPrimitive->SetPhysicsLinearVelocity(Velocity);
		}
	}
}


void UProjectileMovementComponent::UpdateTickRegistration()
{
	if (bAutoUpdateTickRegistration)
	{
		if (!bInterpolationComplete)
		{
			SetComponentTickEnabled(true);
		}
		else
		{
			Super::UpdateTickRegistration();
		}
	}
}

void UProjectileMovementComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	QUICK_SCOPE_CYCLE_COUNTER( STAT_ProjectileMovementComponent_TickComponent );
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ProjectileMovement);

	// Can avoid moving the interpolated object's children until the end of the entire simulation frame.
	// This only makes sense if simulation is also enabled, which would move the UpdatedComponent and move the attached InterpolatedComponent (and children) again.
	const bool bUseScopedInterpolatedMove = bInterpolationUseScopedMovement && bSimulationEnabled;
	const FScopedMovementUpdate ScopedInterpolatedMove(GetInterpolatedComponent(), bUseScopedInterpolatedMove ? EScopedUpdate::DeferredUpdates : EScopedUpdate::ImmediateUpdates);

	// Still need to finish interpolating after we've stopped simulating, so do that first.
	if (!bInterpolationComplete)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ProjectileMovementComponent_TickInterpolation);
		TickInterpolation(DeltaTime);
	}

	// Consume PendingForce and reset to zero.
	// At this point, any calls to AddForce() will apply to the next frame.
	PendingForceThisUpdate = PendingForce;
	ClearPendingForce();

	// skip if don't want component updated when not rendered or updated component can't move
	if (HasStoppedSimulation() || ShouldSkipUpdate(DeltaTime))
	{
		return;
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (!IsValid(UpdatedComponent) || !bSimulationEnabled)
	{
		return;
	}

	AActor* ActorOwner = UpdatedComponent->GetOwner();
	if ( !ActorOwner || !CheckStillInWorld() )
	{
		return;
	}

	if (UpdatedComponent->IsSimulatingPhysics())
	{
		return;
	}

	FVector::FReal VelocityTolerance = 0.0;
	switch (ActorOwner->GetReplicatedMovement().VelocityQuantizationLevel)
	{
	case EVectorQuantization::RoundWholeNumber: VelocityTolerance = 1.0; break;
	case EVectorQuantization::RoundOneDecimal:  VelocityTolerance = 0.1; break;
	case EVectorQuantization::RoundTwoDecimals: VelocityTolerance = 0.01; break;
	default:									VelocityTolerance = UE_KINDA_SMALL_NUMBER; break;
	}
	
	float RemainingTime	= DeltaTime;
	int32 NumImpacts = 0;
	int32 NumBounces = 0;
	int32 LoopCount = 0;
	int32 Iterations = 0;
	FHitResult Hit(1.f);
	
	QUICK_SCOPE_CYCLE_COUNTER( STAT_ProjectileMovementComponent_PerformMovement );
	const FScopedMovementUpdate ScopedProjectileUpdate(bSimulationUseScopedMovement ? UpdatedComponent : nullptr, EScopedUpdate::DeferredUpdates);

	while (bSimulationEnabled && RemainingTime >= MIN_TICK_TIME && (Iterations < MaxSimulationIterations) && IsValid(ActorOwner) && !HasStoppedSimulation())
	{
		LoopCount++;
		Iterations++;

		// subdivide long ticks to more closely follow parabolic trajectory
		const float InitialTimeRemaining = RemainingTime;
		const float TimeTick = ShouldUseSubStepping() ? GetSimulationTimeStep(RemainingTime, Iterations) : RemainingTime;
		RemainingTime -= TimeTick;
		
		// Logging
		UE_LOGF(LogProjectileMovement, Verbose, "Projectile %ls: (Role: %d, Iteration %d, step %.3f, [%.3f / %.3f] cur/total) sim (Pos %ls, Vel %ls)",
			*GetNameSafe(ActorOwner), (int32)ActorOwner->GetLocalRole(), LoopCount, TimeTick, FMath::Max(0.f, DeltaTime - InitialTimeRemaining), DeltaTime,
			*UpdatedComponent->GetComponentLocation().ToString(), *Velocity.ToString());

		// Initial move state
		Hit.Time = 1.f;
		const FVector OldVelocity = Velocity;
		const FVector MoveDelta = ComputeMoveDelta(OldVelocity, TimeTick);
		FQuat NewRotation = (bRotationFollowsVelocity && !OldVelocity.IsNearlyZero(VelocityTolerance)) ? OldVelocity.ToOrientationQuat() : UpdatedComponent->GetComponentQuat();

		if (bRotationFollowsVelocity && bRotationRemainsVertical)
		{
			FRotator DesiredRotation = NewRotation.Rotator();
			DesiredRotation.Pitch = 0.0f;
			DesiredRotation.Yaw = FRotator::NormalizeAxis(DesiredRotation.Yaw);
			DesiredRotation.Roll = 0.0f;
			NewRotation = DesiredRotation.Quaternion();
		}

		// Move the component
		if (bShouldBounce)
		{
			// If we can bounce, we are allowed to move out of penetrations, so use SafeMoveUpdatedComponent which does that automatically.
			SafeMoveUpdatedComponent( MoveDelta, NewRotation, bSweepCollision, Hit );
		}
		else
		{
			// If we can't bounce, then we shouldn't adjust if initially penetrating, because that should be a blocking hit that causes a hit event and stop simulation.
			TGuardValue<EMoveComponentFlags> ScopedFlagRestore(MoveComponentFlags, MoveComponentFlags | MOVECOMP_NeverIgnoreBlockingOverlaps);
			MoveUpdatedComponent(MoveDelta, NewRotation, bSweepCollision, &Hit );
		}
		
		// If we hit a trigger that destroyed us, abort.
		if( !IsValid(ActorOwner) || HasStoppedSimulation() )
		{
			return;
		}

		// Handle hit result after movement
		if( !Hit.bBlockingHit )
		{
			PreviousHitTime = 1.f;
			bIsSliding = false;

			// Only calculate new velocity if events didn't change it during the movement update.
			if (Velocity == OldVelocity)
			{
				Velocity = ComputeVelocity(Velocity, TimeTick);				
			}

			// Logging
			UE_LOGF(LogProjectileMovement, VeryVerbose, "Projectile %ls: (Role: %d, Iteration %d, step %.3f) no hit (Pos %ls, Vel %ls)",
				*GetNameSafe(ActorOwner), (int32)ActorOwner->GetLocalRole(), LoopCount, TimeTick, *UpdatedComponent->GetComponentLocation().ToString(), *Velocity.ToString());
		}
		else
		{
			// Only calculate new velocity if events didn't change it during the movement update.
			if (Velocity == OldVelocity)
			{
				// re-calculate end velocity for partial time
				Velocity = (Hit.Time > UE_KINDA_SMALL_NUMBER) ? ComputeVelocity(OldVelocity, TimeTick * Hit.Time) : OldVelocity;
			}

			// Logging
			UE_CLOGF(UpdatedComponent != nullptr, LogProjectileMovement, VeryVerbose, "Projectile %ls: (Role: %d, Iteration %d, step %.3f) new hit at t=%.3f: (Pos %ls, Vel %ls)",
				*GetNameSafe(ActorOwner), (int32)ActorOwner->GetLocalRole(), LoopCount, TimeTick, Hit.Time, *UpdatedComponent->GetComponentLocation().ToString(), *Velocity.ToString());

			// Handle blocking hit
			NumImpacts++;
			float SubTickTimeRemaining = TimeTick * (1.f - Hit.Time);
			const EHandleBlockingHitResult HandleBlockingResult = HandleBlockingHit(Hit, TimeTick, MoveDelta, SubTickTimeRemaining);
			if (HandleBlockingResult == EHandleBlockingHitResult::Abort || HasStoppedSimulation())
			{
				break;
			}
			else if (HandleBlockingResult == EHandleBlockingHitResult::Deflect)
			{
				NumBounces++;
				HandleDeflection(Hit, OldVelocity, NumBounces, SubTickTimeRemaining);
				PreviousHitTime = Hit.Time;
				PreviousHitNormal = ConstrainNormalToPlane(Hit.Normal);
			}
			else if (HandleBlockingResult == EHandleBlockingHitResult::AdvanceNextSubstep)
			{
				// Reset deflection logic to ignore this hit
				PreviousHitTime = 1.f;
			}
			else
			{
				// Unhandled EHandleBlockingHitResult
				checkNoEntry();
			}
			
			// Logging
			UE_CLOGF(UpdatedComponent != nullptr, LogProjectileMovement, VeryVerbose, "Projectile %ls: (Role: %d, Iteration %d, step %.3f) deflect at t=%.3f: (Pos %ls, Vel %ls)",
				*GetNameSafe(ActorOwner), (int32)ActorOwner->GetLocalRole(), Iterations, TimeTick, Hit.Time, *UpdatedComponent->GetComponentLocation().ToString(), *Velocity.ToString());
			
			// Add unprocessed time after impact
			if (SubTickTimeRemaining >= MIN_TICK_TIME)
			{
				RemainingTime += SubTickTimeRemaining;

				// A few initial impacts should possibly allow more iterations to complete more of the simulation.
				if (NumImpacts <= BounceAdditionalIterations)
				{
					Iterations--;

					// Logging
					UE_LOGF(LogProjectileMovement, Verbose, "Projectile %ls: (Role: %d, Iteration %d, step %.3f) allowing extra iteration after bounce %u (t=%.3f, adding %.3f secs)",
						*GetNameSafe(ActorOwner), (int32)ActorOwner->GetLocalRole(), LoopCount, TimeTick, NumBounces, Hit.Time, SubTickTimeRemaining);
				}
			}
		}
	}

	UpdateComponentVelocity();
}


bool UProjectileMovementComponent::HandleDeflection(FHitResult& Hit, const FVector& OldVelocity, const uint32 NumBounces, float& SubTickTimeRemaining)
{
	const FVector Normal = ConstrainNormalToPlane(Hit.Normal);

	// Multiple hits within very short time period?
	const bool bMultiHit = (PreviousHitTime < 1.f && Hit.Time <= UE_KINDA_SMALL_NUMBER);

	// if velocity still into wall (after HandleBlockingHit() had a chance to adjust), slide along wall
	const float DotTolerance = 0.01f;
	bIsSliding = (bMultiHit && FVector::Coincident(PreviousHitNormal, Normal)) ||
					((Velocity.GetSafeNormal() | Normal) <= DotTolerance);

	if (bIsSliding)
	{
		if (bMultiHit && (PreviousHitNormal | Normal) <= 0.f)
		{
			//90 degree or less corner, so use cross product for direction
			FVector NewDir = (Normal ^ PreviousHitNormal);
			NewDir = NewDir.GetSafeNormal();
			Velocity = Velocity.ProjectOnToNormal(NewDir);
			if ((OldVelocity | Velocity) < 0.f)
			{
				Velocity *= -1.f;
			}
			Velocity = ConstrainDirectionToPlane(Velocity);
		}
		else
		{
			//adjust to move along new wall
			Velocity = ComputeSlideVector(Velocity, 1.f, Normal, Hit);
		}

		// Check min velocity.
		if (IsVelocityUnderSimulationThreshold())
		{
			StopSimulating(Hit);
			return false;
		}

		// Velocity is now parallel to the impact surface.
		if (SubTickTimeRemaining > UE_KINDA_SMALL_NUMBER)
		{
			if (!HandleSliding(Hit, SubTickTimeRemaining))
			{
				return false;
			}
		}
	}

	return true;
}


bool UProjectileMovementComponent::HandleSliding(FHitResult& Hit, float& SubTickTimeRemaining)
{
	FHitResult InitialHit(Hit);
	const FVector OldHitNormal = ConstrainDirectionToPlane(Hit.Normal);

	// Velocity is now parallel to the impact surface.
	// Perform the move now, before adding gravity/accel again, so we don't just keep hitting the surface.
	SafeMoveUpdatedComponent(Velocity * SubTickTimeRemaining, UpdatedComponent->GetComponentQuat(), bSweepCollision, Hit);

	if (HasStoppedSimulation())
	{
		return false;
	}

	// A second hit can deflect the velocity (through the normal bounce code), for the next iteration.
	if (Hit.bBlockingHit)
	{
		const float TimeTick = SubTickTimeRemaining;
		SubTickTimeRemaining = TimeTick * (1.f - Hit.Time);
		
		if (HandleBlockingHit(Hit, TimeTick, Velocity * TimeTick, SubTickTimeRemaining) == EHandleBlockingHitResult::Abort ||
			HasStoppedSimulation())
		{
			return false;
		}
	}
	else
	{
		// Find velocity after elapsed time
		const FVector PostTickVelocity = ComputeVelocity(Velocity, SubTickTimeRemaining);

		// If pointing back into surface, apply friction and acceleration.
		const FVector Force = (PostTickVelocity - Velocity);
		const float ForceDotN = (Force | OldHitNormal);
		if (ForceDotN < 0.f)
		{
			const FVector ProjectedForce = FVector::VectorPlaneProject(Force, OldHitNormal);
			const FVector NewVelocity = Velocity + ProjectedForce;

			const FVector FrictionForce = -NewVelocity.GetSafeNormal() * FMath::Min(-ForceDotN * Friction, NewVelocity.Size());
			Velocity = ConstrainDirectionToPlane(NewVelocity + FrictionForce);
		}
		else
		{
			Velocity = PostTickVelocity;
		}

		// Check min velocity
		if (IsVelocityUnderSimulationThreshold())
		{
			StopSimulating(InitialHit);
			return false;
		}

		SubTickTimeRemaining = 0.f;
	}

	return true;
}


void UProjectileMovementComponent::SetVelocityInLocalSpace(FVector NewVelocity)
{
	if (UpdatedComponent)
	{
		Velocity = UpdatedComponent->GetComponentToWorld().TransformVectorNoScale(NewVelocity);
	}
}


FVector UProjectileMovementComponent::ComputeVelocity(FVector InitialVelocity, float DeltaTime) const
{
	// v = v0 + a*t
	const FVector Acceleration = ComputeAcceleration(InitialVelocity, DeltaTime);
	FVector NewVelocity = InitialVelocity + (Acceleration * DeltaTime);

	return LimitVelocity(NewVelocity);
}


FVector UProjectileMovementComponent::LimitVelocity(FVector NewVelocity) const
{
	const float CurrentMaxSpeed = GetMaxSpeed();
	if (CurrentMaxSpeed > 0.f)
	{
		NewVelocity = NewVelocity.GetClampedToMaxSize(CurrentMaxSpeed);
	}

	return ConstrainDirectionToPlane(NewVelocity);
}

FVector UProjectileMovementComponent::ComputeMoveDelta(const FVector& InVelocity, float DeltaTime) const
{
	// Velocity Verlet integration (http://en.wikipedia.org/wiki/Verlet_integration#Velocity_Verlet)
	// The addition of p0 is done outside this method, we are just computing the delta.
	// p = p0 + v0*t + 1/2*a*t^2

	// We use ComputeVelocity() here to infer the acceleration, to make it easier to apply custom velocities.
	// p = p0 + v0*t + 1/2*((v1-v0)/t)*t^2
	// p = p0 + v0*t + 1/2*((v1-v0))*t

	const FVector NewVelocity = ComputeVelocity(InVelocity, DeltaTime);
	const FVector Delta = (InVelocity * DeltaTime) + (NewVelocity - InVelocity) * (0.5f * DeltaTime);
	return Delta;
}

FVector UProjectileMovementComponent::ComputeAcceleration(const FVector& InVelocity, float DeltaTime) const
{
	FVector Acceleration(FVector::ZeroVector);

	Acceleration.Z += GetGravityZ();

	Acceleration += PendingForceThisUpdate;

	if (bIsHomingProjectile && HomingTargetComponent.IsValid())
	{
		Acceleration += ComputeHomingAcceleration(InVelocity, DeltaTime);
	}

	return Acceleration;
}

// Allow the projectile to track towards its homing target.
FVector UProjectileMovementComponent::ComputeHomingAcceleration(const FVector& InVelocity, float DeltaTime) const
{
	FVector HomingAcceleration = ((HomingTargetComponent->GetComponentLocation() - UpdatedComponent->GetComponentLocation()).GetSafeNormal() * HomingAccelerationMagnitude);
	return HomingAcceleration;
}


void UProjectileMovementComponent::AddForce(FVector Force)
{
	PendingForce += Force;
}

void UProjectileMovementComponent::ClearPendingForce(bool bClearImmediateForce)
{
	PendingForce = FVector::ZeroVector;
	if (bClearImmediateForce)
	{
		PendingForceThisUpdate = FVector::ZeroVector;
	}
}

float UProjectileMovementComponent::GetGravityZ() const
{
	// TODO: apply buoyancy if in water
	return ShouldApplyGravity() ? Super::GetGravityZ() * ProjectileGravityScale : 0.f;
}


void UProjectileMovementComponent::StopSimulating(const FHitResult& HitResult)
{
	Velocity = FVector::ZeroVector;
	PendingForce = FVector::ZeroVector;
	PendingForceThisUpdate = FVector::ZeroVector;
	UpdateComponentVelocity();
	SetUpdatedComponent(NULL);
	OnProjectileStop.Broadcast(HitResult);
}


UProjectileMovementComponent::EHandleBlockingHitResult UProjectileMovementComponent::HandleBlockingHit(const FHitResult& Hit, float TimeTick, const FVector& MoveDelta, float& SubTickTimeRemaining)
{
	AActor* ActorOwner = UpdatedComponent ? UpdatedComponent->GetOwner() : NULL;
	if (!CheckStillInWorld() || !IsValid(ActorOwner))
	{
		return EHandleBlockingHitResult::Abort;
	}
	
	HandleImpact(Hit, TimeTick, MoveDelta);
	
	if (!IsValid(ActorOwner) || HasStoppedSimulation())
	{
		return EHandleBlockingHitResult::Abort;
	}

	if (Hit.bStartPenetrating)
	{
		UE_LOGF(LogProjectileMovement, Verbose, "Projectile %ls is stuck inside %ls.%ls with velocity %ls!", *GetNameSafe(ActorOwner), *Hit.HitObjectHandle.GetName(), *GetNameSafe(Hit.GetComponent()), *Velocity.ToString());
		return EHandleBlockingHitResult::Abort;
	}

	SubTickTimeRemaining = TimeTick * (1.f - Hit.Time);
	return EHandleBlockingHitResult::Deflect;
}
 
FVector UProjectileMovementComponent::ComputeBounceResult(const FHitResult& Hit, float TimeSlice, const FVector& MoveDelta)
{
	FVector TempVelocity = Velocity;
	const FVector Normal = ConstrainNormalToPlane(Hit.Normal);
	const float VDotNormal = (TempVelocity | Normal);

	// Only if velocity is opposed by normal or parallel
	if (VDotNormal <= 0.f)
	{
		// Project velocity onto normal in reflected direction.
		const FVector ProjectedNormal = Normal * -VDotNormal;

		// Point velocity in direction parallel to surface
		TempVelocity += ProjectedNormal;

		// Only tangential velocity should be affected by friction.
		const float ScaledFriction = (bBounceAngleAffectsFriction || bIsSliding) ? FMath::Clamp(-VDotNormal / TempVelocity.Size(), MinFrictionFraction, 1.f) * Friction : Friction;
		TempVelocity *= FMath::Clamp(1.f - ScaledFriction, 0.f, 1.f);

		// Coefficient of restitution only applies perpendicular to impact.
		TempVelocity += (ProjectedNormal * FMath::Max(Bounciness, 0.f));

		// Bounciness could cause us to exceed max speed.
		TempVelocity = LimitVelocity(TempVelocity);
	}

	return TempVelocity;
}

void UProjectileMovementComponent::HandleImpact(const FHitResult& Hit, float TimeSlice, const FVector& MoveDelta)
{
	bool bStopSimulating = false;

	if (bShouldBounce)
	{
		const FVector OldVelocity = Velocity;
		Velocity = ComputeBounceResult(Hit, TimeSlice, MoveDelta);

		// Trigger bounce events
		OnProjectileBounce.Broadcast(Hit, OldVelocity);

		// Event may modify velocity or threshold, so check velocity threshold now.
		Velocity = LimitVelocity(Velocity);
		if (IsVelocityUnderSimulationThreshold())
		{
			bStopSimulating = true;
		}
	}
	else
	{
		bStopSimulating = true;
	}


	if (bStopSimulating)
	{
		StopSimulating(Hit);
	}
}

bool UProjectileMovementComponent::CheckStillInWorld()
{
	if ( !UpdatedComponent )
	{
		return false;
	}

	const UWorld* MyWorld = GetWorld();
	if (!MyWorld)
	{
		return false;
	}

	// check the variations of KillZ
	AWorldSettings* WorldSettings = MyWorld->GetWorldSettings( true );
	if (!WorldSettings->AreWorldBoundsChecksEnabled())
	{
		return true;
	}
	AActor* ActorOwner = UpdatedComponent->GetOwner();
	if (!IsValid(ActorOwner))
	{
		return false;
	}
	if( ActorOwner->GetActorLocation().Z < WorldSettings->KillZ )
	{
		UDamageType const* DmgType = WorldSettings->KillZDamageType ? WorldSettings->KillZDamageType->GetDefaultObject<UDamageType>() : GetDefault<UDamageType>();
		ActorOwner->FellOutOfWorld(*DmgType);
		return false;
	}
	// Check if box has poked outside the world
	else if( UpdatedComponent && UpdatedComponent->IsRegistered() )
	{
		const FBox&	Box = UpdatedComponent->Bounds.GetBox();
		if(	Box.Min.X < -HALF_WORLD_MAX || Box.Max.X > HALF_WORLD_MAX ||
			Box.Min.Y < -HALF_WORLD_MAX || Box.Max.Y > HALF_WORLD_MAX ||
			Box.Min.Z < -HALF_WORLD_MAX || Box.Max.Z > HALF_WORLD_MAX )
		{
			UE_LOGF(LogProjectileMovement, Warning, "%ls is outside the world bounds!", *ActorOwner->GetName());
			ActorOwner->OutsideWorldBounds();
			// not safe to use physics or collision at this point
			ActorOwner->SetActorEnableCollision(false);
			FHitResult Hit(1.f);
			StopSimulating(Hit);
			return false;
		}
	}
	return true;
}


bool UProjectileMovementComponent::ShouldUseSubStepping() const
{
	return bForceSubStepping || (GetGravityZ() != 0.f) || (bIsHomingProjectile && HomingTargetComponent.IsValid());
}


float UProjectileMovementComponent::GetSimulationTimeStep(float RemainingTime, int32 Iterations) const
{
	if (RemainingTime > MaxSimulationTimeStep)
	{
		if (Iterations < MaxSimulationIterations)
		{
			// Subdivide moves to be no longer than MaxSimulationTimeStep seconds
			RemainingTime = FMath::Min(MaxSimulationTimeStep, RemainingTime * 0.5f);
		}
		else
		{
			// If this is the last iteration, just use all the remaining time. This is better than cutting things short, as the simulation won't move far enough otherwise.
			// Print a throttled warning.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (const UWorld* const World = GetWorld())
			{
				// Don't report during long hitches, we're more concerned about normal behavior just to make sure we have reasonable simulation settings.
				if (World->DeltaTimeSeconds < 0.20f)
				{
					static uint32 s_WarningCount = 0;
					if ((s_WarningCount++ < 100) || (GFrameCounter & 15) == 0)
					{
						UE_LOGF(LogProjectileMovement, Warning, "GetSimulationTimeStep() - Max iterations %d hit while remaining time %.6f > MaxSimulationTimeStep (%.3f) for '%ls'", MaxSimulationIterations, RemainingTime, MaxSimulationTimeStep, *GetPathNameSafe(UpdatedComponent));
					}
				}
			}
#endif
		}
	}

	// no less than MIN_TICK_TIME (to avoid potential divide-by-zero during simulation).
	return FMath::Max(MIN_TICK_TIME, RemainingTime);
}

void UProjectileMovementComponent::SetInterpolatedComponent(USceneComponent* Component)
{
	if (Component == GetInterpolatedComponent())
	{
		return;
	}

	if (Component)
	{
		if (!ensureMsgf(Component != UpdatedComponent, TEXT("ProjectileMovement interpolated component should not be the same as the simulated component.")))
		{
			return;
		}

		ResetInterpolation();
		InterpolatedComponentPtr = Component;
		InterpInitialLocationOffset = Component->GetRelativeLocation();
		InterpInitialRotationOffset = Component->GetRelativeRotation().Quaternion();
		// We start at the "completed" location, wait for MoveInterpolationTarget() to actually mark it dirty.
		bInterpolationComplete = true;

		// Space out interpolation skipping to avoid objects spawned on single frame from always updating in sync
		ThrottleInterpolationFramesSinceInterp = ThrottleInterpolationSkipFramesRecent > 0 ? FMath::RandRange(0, ThrottleInterpolationSkipFramesRecent) : 0;
	}
	else
	{
		ResetInterpolation();
		InterpolatedComponentPtr = nullptr;
		InterpInitialLocationOffset = FVector::ZeroVector;
		InterpInitialRotationOffset = FQuat::Identity;
		bInterpolationComplete = true;
		// Disabling interpolation should stop our ticking if we are done simulating and just trying to finish interpolation.
		if (bAutoUpdateTickRegistration && (UpdatedComponent == nullptr))
		{
			UpdateTickRegistration();
		}
	}
}

USceneComponent* UProjectileMovementComponent::GetInterpolatedComponent() const
{
	return InterpolatedComponentPtr.Get();
}

void UProjectileMovementComponent::MoveInterpolationTarget(const FVector& NewLocation, const FRotator& NewRotation)
{
	if (!UpdatedComponent)
	{
		return;
	}

	bool bHandledMovement = false;
	if (bInterpMovement)
	{
		if (USceneComponent* InterpComponent = GetInterpolatedComponent())
		{
			// Avoid moving the child, it will interpolate later
			const FRotator InterpRelativeRotation = InterpComponent->GetRelativeRotation();
			FScopedPreventAttachedComponentMove ScopedChildNoMove(InterpComponent);
			
			// Update interp offset
			const FVector OldLocation = UpdatedComponent->GetComponentLocation();
			const FVector NewToOldVector = (OldLocation - NewLocation);
			InterpLocationOffset += NewToOldVector;

			// Enforce distance limits
			if (NewToOldVector.SizeSquared() > FMath::Square(InterpLocationSnapToTargetDistance))
			{
				InterpLocationOffset = FVector::ZeroVector;
			}
			else if (InterpLocationOffset.SizeSquared() > FMath::Square(InterpLocationMaxLagDistance))
			{
				InterpLocationOffset = InterpLocationMaxLagDistance * InterpLocationOffset.GetSafeNormal();
			}

			// Handle rotation
			if (bInterpRotation)
			{
				const FQuat OldRotation = UpdatedComponent->GetComponentQuat();
				InterpRotationOffset = (NewRotation.Quaternion().Inverse() * OldRotation) * InterpRotationOffset;
			}
			else
			{
				// If not interpolating rotation, we should allow the component to rotate.
				// The absolute flag will get restored by the scoped move.
				InterpComponent->SetUsingAbsoluteRotation(false);
				InterpComponent->SetRelativeRotation_Direct(InterpRelativeRotation);
				InterpRotationOffset = FQuat::Identity;
			}

			// Move the root
			UpdatedComponent->SetRelativeLocationAndRotation(NewLocation, NewRotation);
			bHandledMovement = true;
			bInterpolationComplete = false;
		}
		else
		{
			ResetInterpolation();
			bInterpolationComplete = true;
		}
	}
	
	if (!bHandledMovement)
	{
		UpdatedComponent->SetRelativeLocationAndRotation(NewLocation, NewRotation);
	}
}

void UProjectileMovementComponent::ResetInterpolation()
{
	if (USceneComponent* InterpComponent = GetInterpolatedComponent())
	{
		// Snap to original (non-interpolated) offset, we may be forcibly stopping interpolation and need to have it stay at the correct location.
		InterpComponent->SetRelativeLocationAndRotation(InterpInitialLocationOffset, InterpInitialRotationOffset);
	}

	InterpLocationOffset = FVector::ZeroVector;
	InterpRotationOffset = FQuat::Identity;
	bInterpolationComplete = true;
	
	ThrottleInterpolationFramesSinceInterp = 0;
}

void UProjectileMovementComponent::TickInterpolation(float DeltaTime)
{
	if (!bInterpolationComplete)
	{
		if (bInterpMovement)
		{
			// Smooth location. Interp faster when stopping.
			const float ActualInterpLocationTime = Velocity.IsZero() ? 0.5f * InterpLocationTime : InterpLocationTime;
			if (DeltaTime < ActualInterpLocationTime)
			{
				// Slowly decay translation offset (lagged exponential smoothing)
				InterpLocationOffset = (InterpLocationOffset * (1.f - DeltaTime / ActualInterpLocationTime));
			}
			else
			{
				InterpLocationOffset = FVector::ZeroVector;
			}

			// Smooth rotation
			if (DeltaTime < InterpRotationTime && bInterpRotation)
			{
				// Slowly decay rotation offset
				InterpRotationOffset = FQuat::FastLerp(InterpRotationOffset, FQuat::Identity, DeltaTime / InterpRotationTime).GetNormalized();
			}
			else
			{
				InterpRotationOffset = FQuat::Identity;
			}

			// Test for reaching the end
			if (InterpLocationOffset.IsNearlyZero(1e-2f) && InterpRotationOffset.Equals(FQuat::Identity, 1e-5f))
			{
				InterpLocationOffset = FVector::ZeroVector;
				InterpRotationOffset = FQuat::Identity;
				bInterpolationComplete = true;
			}

			if (USceneComponent* InterpComponent = GetInterpolatedComponent())
			{
				const bool bShouldThrottleNow = UpdateThrottleInterpolation(DeltaTime, InterpComponent);
				if (bShouldThrottleNow)
				{
					UE_LOGF(LogProjectileMovementInterpolation, Verbose, "--- Skip  Interpolation (%d frames : %ls)", ThrottleInterpolationFramesSinceInterp, *GetPathNameSafe(InterpComponent));
					// Skip applying transform to InterpolatedComponent.
					// Don't say we're done interpolating if we haven't applied the result yet, we need it to update next frame.
					bInterpolationComplete = false;
				}
				else
				{
					ThrottleInterpolationFramesSinceInterp = 0;
					UE_LOGF(LogProjectileMovementInterpolation, Verbose, "+++ Apply Interpolation (%d frames : %ls)", ThrottleInterpolationFramesSinceInterp, *GetPathNameSafe(InterpComponent));

					// Apply interpolation result
					if (UpdatedComponent)
					{
						const FVector NewRelTranslation = UpdatedComponent->GetComponentToWorld().InverseTransformVectorNoScale(InterpLocationOffset) + InterpInitialLocationOffset;
						if (bInterpRotation)
						{
							const FQuat NewRelRotation = InterpRotationOffset * InterpInitialRotationOffset;
							InterpComponent->SetRelativeLocationAndRotation(NewRelTranslation, NewRelRotation);
						}
						else
						{
							InterpComponent->SetRelativeLocation(NewRelTranslation);
						}
					}
				}
			}
		}
		else
		{
			ResetInterpolation();
			bInterpolationComplete = true;
		}

		// Might be done interpolating and want to disable tick
		if (bInterpolationComplete && bAutoUpdateTickRegistration && (UpdatedComponent == nullptr))
		{
			UpdateTickRegistration();
		}
	}
}


bool UProjectileMovementComponent::UpdateThrottleInterpolation(float DeltaTime, USceneComponent* InterpComponent)
{
	bool bIsThrottlingThisFrame = false;

	if (bThrottleInterpolation && bInterpMovement && InterpComponent)
	{
		const int32 ThrottleFrames = ComputeThrottleInterpolationMaxFrames(DeltaTime, InterpComponent);
		if (ThrottleFrames > 0)
		{
			ThrottleInterpolationFramesSinceInterp += 1;
			if (ThrottleInterpolationFramesSinceInterp <= ThrottleFrames)
			{
				bIsThrottlingThisFrame = true;
			}
		}
	}

	// Detect transition from throttled to not throttled.
	if (!bIsThrottlingThisFrame && ThrottleInterpolationFramesSinceInterp > 0)
	{
		// Reset counter, not throttling this frame.
		ThrottleInterpolationFramesSinceInterp = 0;

		// Hook for custom reset logic
		ResetThrottleInterpolation(DeltaTime);
	}

	return bIsThrottlingThisFrame;
}

int32 UProjectileMovementComponent::ComputeThrottleInterpolationMaxFrames(float DeltaTime, USceneComponent* InterpComponent)
{
	int32 ThrottleFrames = 0;
	if (AActor* ActorOwner = InterpComponent->GetOwner())
	{
		// Not recently rendered?
		if (!ActorOwner->WasRecentlyRendered(ThrottleInterpolationThresholdNotRenderedShortTime))
		{
			// Not rendered even a long time ago?
			if (!ActorOwner->WasRecentlyRendered(ThrottleInterpolationThresholdNotRenderedLongTime))
			{
				ThrottleFrames = ThrottleInterpolationSkipFramesNotRecent;
			}
			else
			{
				ThrottleFrames = ThrottleInterpolationSkipFramesRecent;
			}
		}
	}

	return ThrottleFrames;
}

void UProjectileMovementComponent::ResetThrottleInterpolation(float DeltaTime)
{
	ThrottleInterpolationFramesSinceInterp = 0;
}

