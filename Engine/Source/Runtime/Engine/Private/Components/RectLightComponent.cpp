// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RectLightComponent.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Texture2D.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "RectLightSceneProxy.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RectLightComponent)

#define LOCTEXT_NAMESPACE "RectLightComponent"

float GetRectLightBarnDoorMaxAngle()
{
	return 88.f;
}

static float SolveQuadraticEq(float A, float B, float C)
{
	float Disc = B * B - 4.0f * C * A;
	if (Disc > UE_KINDA_SMALL_NUMBER)
	{
		Disc = FMath::Sqrt(Disc);
		const float Denom = 1.0f / (2.0f * A);
		float Root0 = (-B + Disc) * Denom;
		float Root1 = (-B - Disc) * Denom;

		return Root0;
	}

	return -1;
}

void CalculateRectLightCullingBarnExtentAndDepth(float Size, float Length, float AngleRad, float Radius, float& OutExtent, float& OutDepth)
{
	float T = Size / 2.0f;

	// 1. calculate opposite side (law of cosines)
	float A = Size;
	float B = Length;
	float C = FMath::Sqrt(A * A + B * B - 2 * A * B * FMath::Cos(AngleRad + UE_HALF_PI));

	// 2. calculate angle between rect plane and shadow boundary (law of sines)
	float AuxAngleRad = FMath::Asin(B * FMath::Sin(AngleRad + UE_HALF_PI) / C);

	// 3. calculate shadow boundary line
	float M = FMath::Tan(AuxAngleRad);
	float K = M * T;

	// 4. intersect shadow boundary line with circle
	float X = SolveQuadraticEq(M * M + 1, 2 * M * K, K * K - Radius * Radius);
	float Y = M * X + K;

	if (FMath::Sqrt((X + T) * (X + T) + Y * Y) >= C)
	{
		OutExtent = X - T;
		OutDepth = Y;
	}
	else
	{
		// if intersection is closer than regular barn doors, fallback to base extent / depth 
		OutExtent = FMath::Sin(AngleRad) * Length;
		OutDepth = FMath::Cos(AngleRad) * Length;
	}
}

void CalculateRectLightBarnCorners(float SourceWidth, float SourceHeight, float BarnExtent, float BarnDepth, TStaticArray<FVector, 8>& OutCorners)
{
	OutCorners[0] = FVector(0.0f, +0.5f * SourceWidth, +0.5f * SourceHeight);
	OutCorners[1] = FVector(0.0f, +0.5f * SourceWidth, -0.5f * SourceHeight);
	OutCorners[2] = FVector(BarnDepth, +0.5f * SourceWidth + BarnExtent, +0.5f * SourceHeight + BarnExtent);
	OutCorners[3] = FVector(BarnDepth, +0.5f * SourceWidth + BarnExtent, -0.5f * SourceHeight - BarnExtent);
	OutCorners[4] = FVector(0.0f, -0.5f * SourceWidth, +0.5f * SourceHeight);
	OutCorners[5] = FVector(0.0f, -0.5f * SourceWidth, -0.5f * SourceHeight);
	OutCorners[6] = FVector(BarnDepth, -0.5f * SourceWidth - BarnExtent, +0.5f * SourceHeight + BarnExtent);
	OutCorners[7] = FVector(BarnDepth, -0.5f * SourceWidth - BarnExtent, -0.5f * SourceHeight - BarnExtent);
}

URectLightComponent::URectLightComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UTexture2D> StaticTexture(TEXT("/Engine/EditorResources/LightIcons/S_LightRect"));
		static ConstructorHelpers::FObjectFinder<UTexture2D> DynamicTexture(TEXT("/Engine/EditorResources/LightIcons/S_LightRect"));

		StaticEditorTexture = StaticTexture.Object;
		StaticEditorTextureScale = 0.5f;
		DynamicEditorTexture = DynamicTexture.Object;
		DynamicEditorTextureScale = 0.5f;
	}
#endif

	SourceWidth = 64.0f;
	SourceHeight = 64.0f;
	SourceTexture = nullptr;
	SourceTextureOffset = FVector2f(0.0f, 0.0f);
	SourceTextureScale = FVector2f(1.0f, 1.0f);
	BarnDoorAngle = GetRectLightBarnDoorMaxAngle();
	BarnDoorLength = 20.0f;
	LightFunctionConeAngle = 0.0f;
}

FLightSceneProxy* URectLightComponent::CreateSceneProxy() const
{
	if (!IsPSOPrecaching())
	{
		return new FRectLightSceneProxy(this);
	}
	return nullptr;
}

void URectLightComponent::SetSourceTexture(UTexture* NewValue)
{
	if (NewValue && NewValue->VirtualTextureStreaming)
	{
		// Virtual textures aren't supported.
		// We could add support in future, but we would need to force stream them before uploading to the RectLight Atlas so there would be little benefit for the added complexity.
		UE_LOGF(LogTemp, Warning, "RectLightComponent (%ls) doesn't support Virtual Textures (%ls).", *GetName(), *NewValue->GetName())
		return;
	}

	if (AreDynamicDataChangesAllowed()
		&& SourceTexture != NewValue)
	{
		SourceTexture = NewValue;

		// This will trigger a recreation of the LightSceneProxy
		MarkRenderStateDirty();
	}
}

void URectLightComponent::SetSourceWidth(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& SourceWidth != NewValue)
	{
		SourceWidth = NewValue;
		MarkRenderStateDirty();
	}
}

void URectLightComponent::SetSourceHeight(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& SourceHeight != NewValue)
	{
		SourceHeight = NewValue;
		MarkRenderStateDirty();
	}
}

void URectLightComponent::SetBarnDoorLength(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& BarnDoorLength != NewValue)
	{
		BarnDoorLength = FMath::Max(NewValue, 0.1f);
		MarkRenderStateDirty();
	}
}

void URectLightComponent::SetBarnDoorAngle(float NewValue)
{
	if (AreDynamicDataChangesAllowed()
		&& BarnDoorAngle != NewValue)
	{
		const float MaxAngle = GetRectLightBarnDoorMaxAngle();
		BarnDoorAngle = FMath::Clamp(NewValue, 0.f, MaxAngle);
		MarkRenderStateDirty();
	}
}

float URectLightComponent::ComputeLightBrightness() const
{
	float LightBrightness = Super::ComputeLightBrightness();

	if (IntensityUnits == ELightUnits::Candelas)
	{
		LightBrightness *= (100.f * 100.f); // Conversion from m2 to cm2
	}
	else if (IntensityUnits == ELightUnits::Nits)
	{
		const float AreaInCm2 = SourceWidth * SourceHeight;
		LightBrightness *= AreaInCm2;
	}
	else if (IntensityUnits == ELightUnits::Lumens)
	{
		LightBrightness *= (100.f * 100.f / UE_PI); // Conversion from cm2 to m2 and PI from the cosine distribution
	}
	else if (IntensityUnits == ELightUnits::EV)
	{
		if (bLightRequiresBrokenEVMath)
		{
			// The code below is a typo, but to preserve legacy content, we need to maintain it so that old scenes
			// keep working even in cases with blueprint logic, sequencer animations, etc ... which cannot be fixed
			// trivially via serialization.
			LightBrightness *= EV100ToLuminance(LightBrightness) * (100.f * 100.f);
		}
		else
		{
			// This is the correct formula
			LightBrightness = EV100ToLuminance(LightBrightness) * (100.f * 100.f);
		}
	}
	else
	{
		LightBrightness *= 16; // Legacy scale of 16
	}

	return LightBrightness;
}

#if WITH_EDITOR
void URectLightComponent::SetLightBrightness(float InBrightness)
{
	if (IntensityUnits == ELightUnits::Candelas)
	{
		Super::SetLightBrightness(InBrightness / (100.f * 100.f)); // Conversion from cm2 to m2
	}
	else if (IntensityUnits == ELightUnits::Nits)
	{
		const float AreaInCm2 = SourceWidth * SourceHeight;
		Super::SetLightBrightness(InBrightness / AreaInCm2);
	}
	else if (IntensityUnits == ELightUnits::Lumens)
	{
		Super::SetLightBrightness(InBrightness / (100.f * 100.f / UE_PI)); // Conversion from cm2 to m2 and PI from the cosine distribution
	}
	else if (IntensityUnits == ELightUnits::EV)
	{
		Super::SetLightBrightness(LuminanceToEV100(InBrightness / (100.f * 100.f)));
	}
	else
	{
		Super::SetLightBrightness(InBrightness / 16); // Legacy scale of 16
	}
}

bool URectLightComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(URectLightComponent, LightFunctionConeAngle))
		{
			if (Mobility == EComponentMobility::Static)
			{
				return false;
			}
			return LightFunctionMaterial != NULL;
		}
	}

	return Super::CanEditChange(InProperty);
}

void URectLightComponent::CheckForErrors()
{
	Super::CheckForErrors();

	ValidateTexture();
}

void URectLightComponent::ValidateTexture() const
{
	if (SourceTexture && SourceTexture->VirtualTextureStreaming)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("SourceTexture"), FText::FromString(SourceTexture->GetName()));

		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("RectLight_VirtualTextureWarning", "Source Texture {SourceTexture} is a Virtual Texture.\n This is unsupported for Rect Lights and won't appear correctly in cooked builds."), Arguments)));
	}
}

bool URectLightComponent::ShouldFilterSourceTexture(const FAssetData& InAssetData) const
{
	const FName VirtualTexturePropertyName("VirtualTextureStreaming");
	const bool VirtualTextureStreaming = InAssetData.GetTagValueRef<bool>(VirtualTexturePropertyName);
	return VirtualTextureStreaming != 0;
}
#endif // WITH_EDITOR

/**
* @return ELightComponentType for the light component class 
*/
ELightComponentType URectLightComponent::GetLightType() const
{
	return LightType_Rect;
}

float URectLightComponent::GetUniformPenumbraSize() const
{
	if (LightmassSettings.bUseAreaShadowsForStationaryLight)
	{
		// Interpret distance as shadow factor directly
		return 1.0f;
	}
	else
	{
		float SourceRadius = FMath::Sqrt( SourceWidth * SourceHeight );
		// Heuristic to derive uniform penumbra size from light source radius
		return FMath::Clamp(SourceRadius == 0 ? .05f : SourceRadius * .005f, .0001f, 1.0f);
	}
}

void URectLightComponent::BeginDestroy()
{
	Super::BeginDestroy();
}

#if WITH_EDITOR
/**
 * Called after property has changed via e.g. property window or set command.
 *
 * @param	PropertyThatChanged	FProperty that has been changed, NULL if unknown
 */
void URectLightComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	SourceWidth  = FMath::Max(1.0f, SourceWidth);
	SourceHeight = FMath::Max(1.0f, SourceHeight);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR


void URectLightComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::RectLightFixedEVUnitConversion)
	{
		// Before this version, the lights contained a subtly wrong interpretation of EV units (see ComputeLightBrighness() above). To preserve
		// backwards compatibility, we cannot simply change the intensity here (as it would not address other way the intensity can be set such
		// as from blueprints, sequencer, etc ...). Instead, make sure that older lights that come in with EV units just apply the old formula.
		// Limit this fix to lights with units that were explicitly configured to use EV so that older lights will get the correct behavior if
		// their units are changed later. Technically a light that is saved on disk in one unit and dynamically changed to EV in blueprint code
		// will be broken, but this seems like a rare enough case and minimizing the number of files that have this workaround boolean set is
		// preferable.
		if (IntensityUnits == ELightUnits::EV)
		{
			bLightRequiresBrokenEVMath = true;
		}
	}
}

void URectLightComponent::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	ValidateTexture();
#endif
}

void URectLightComponent::BuildSceneProxyDesc(const URectLightComponent& InLightComponent, FRectLightSceneProxyDesc& OutProxyDesc)
{
	ULocalLightComponent::BuildSceneProxyDesc(InLightComponent, OutProxyDesc);

	OutProxyDesc.SourceWidth = InLightComponent.SourceWidth;
	OutProxyDesc.SourceHeight = InLightComponent.SourceHeight;
	OutProxyDesc.BarnDoorAngle = InLightComponent.BarnDoorAngle;
	OutProxyDesc.BarnDoorLength = InLightComponent.BarnDoorLength;
	OutProxyDesc.LightFunctionConeAngle = InLightComponent.LightFunctionConeAngle;
	OutProxyDesc.SourceTexture = InLightComponent.SourceTexture;
	OutProxyDesc.SourceTextureScale = InLightComponent.SourceTextureScale;
	OutProxyDesc.SourceTextureOffset = InLightComponent.SourceTextureOffset;
}

#undef LOCTEXT_NAMESPACE
