// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	
=============================================================================*/

#include "Components/ReflectionCaptureComponent.h"
#include "Misc/Compression.h"
#include "UObject/Package.h"
#include "RenderUtils.h"
#include "UObject/ReflectionCaptureObjectVersion.h"
#include "SceneInterface.h"
#include "SceneProxies/ReflectionCaptureProxy.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Stats/StatsTrace.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/BillboardComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Texture2D.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/ReflectionCapture.h"
#include "EngineModule.h"
#include "Serialization/MemoryReader.h"
#include "ShaderCompiler.h"
#include "Engine/SphereReflectionCapture.h"
#include "Components/SphereReflectionCaptureComponent.h"
#include "Components/DrawSphereComponent.h"
#include "Components/BoxReflectionCaptureComponent.h"
#include "Engine/PlaneReflectionCapture.h"
#include "Engine/BoxReflectionCapture.h"
#include "EngineUtils.h"
#include "Components/PlaneReflectionCaptureComponent.h"
#include "Components/BoxComponent.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Math/PackedVector.h"
#include "GlobalRenderResources.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/UObjectIterator.h"
#include "DataDrivenShaderPlatformInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReflectionCaptureComponent)

#if WITH_EDITOR
#include "TextureCompiler.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogReflectionCaptureComponent, Log, All);

/** 
 * Size of all reflection captures.
 * Reflection capture derived data versions must be changed if modifying this
 */
ENGINE_API TAutoConsoleVariable<int32> CVarReflectionCaptureSize(
	TEXT("r.ReflectionCaptureResolution"),
	128,
	TEXT("Set the resolution for all reflection capture cubemaps. Should be set via project's Render Settings. Must be power of 2. Defaults to 128.\n")
	);

TAutoConsoleVariable<int32> CVarReflectionCaptureUpdateEveryFrame(
	TEXT("r.ReflectionCaptureUpdateEveryFrame"),
	0,
	TEXT("When set, reflection captures will constantly be scheduled for update.\n")
	TEXT("0:  Disabled.\n")
	TEXT("1:  Update all every frame.\n")
	TEXT("2:  Update selected every frame.\n")
);

// Refreshes the render state for captures when the runtime mode changes, either the CVar or the setting on the component.
// Before the render state refresh, we release the capture's slot in the cubemap array, as active captures disabled by the mode
// change are no longer going to be rendered.  Releasing the render state otherwise doesn't free up slots, as render state
// re-create preserves cached slots.  ReleaseReflectionCubemap is a nop for captures that already don't have a slot.
//
// We also queue either type of capture for the upload path.  Runtime captures don't have MapBuildData to upload, but that
// queue is what feeds WorldCombinedCaptures in UpdateReflectionCaptureContents, which in turn is what gets captures into
// AllocatedReflectionCapturesGameThread.  The queue-processing in UpdateReflectionCaptureContents checks for SceneProxy
// before adding to WorldCombinedCaptures, so captures that lost their proxy due to a mode change are dropped.
static void ReflectionCaptureRuntimeModeChanged(UReflectionCaptureComponent* Component)
{
	// Plane reflection captures are a separate system that doesn't support runtime capture, so we leave them alone.
	if (Component->IsA<UPlaneReflectionCaptureComponent>())
	{
		return;
	}

	if (UWorld* World = Component->GetWorld())
	{
		if (FSceneInterface* Scene = World->Scene)
		{
			Scene->ReleaseReflectionCubemap(Component);
		}
	}
	Component->MarkRenderStateDirty();
	Component->MarkDirtyForRecaptureOrUpload();
}

TAutoConsoleVariable<int32> CVarReflectionCaptureRuntime(
	TEXT("r.ReflectionCapture.Runtime"),
	0,
	TEXT("Controls which capture type is used.  Components whose bRuntimeCapture flag doesn't match this mode don't render.\n")
	TEXT("0: Baked captures only (default).  Components flagged as runtime are skipped.\n")
	TEXT("1: Runtime captures only.  Baked components are skipped.  See also r.ReflectionCapture.CookBakedCaptures CVar.\n"),

	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* /*Variable*/)
	{
		// Any capture's proxy creation depends on the CVar, so every capture needs its render state refreshed on a toggle.  
		for (TObjectIterator<UReflectionCaptureComponent> It; It; ++It)
		{
			ReflectionCaptureRuntimeModeChanged(*It);
		}
	})
);

static int32 SanitizeReflectionCaptureSize(int32 ReflectionCaptureSize)
{
	const int32 MaxReflectionCaptureSize = GetMaxCubeTextureDimension();
	const int32 MinReflectionCaptureSize = 1;

	return FMath::Clamp(ReflectionCaptureSize, MinReflectionCaptureSize, MaxReflectionCaptureSize);
}

int32 UReflectionCaptureComponent::GetReflectionCaptureSize()
{
	return SanitizeReflectionCaptureSize(CVarReflectionCaptureSize.GetValueOnAnyThread());
}

void UReflectionCaptureComponent::RefreshCapture(bool bFastRender, bool bSmoothBlend)
{
	if (!IsRuntimeCapture())
	{
		UE_LOGF(LogReflectionCaptureComponent, Warning, "RefreshCapture called on '%ls' which is not a runtime capture (bRuntimeCapture=false). Refresh is a no-op for baked captures.", *GetPathName());
		return;
	}

	if (bSmoothBlend)
	{
		static const auto CVarSmoothBlendSlots = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ReflectionCapture.Runtime.SmoothBlendSlots"));
		if (!CVarSmoothBlendSlots || CVarSmoothBlendSlots->GetValueOnGameThread() == 0)
		{
			UE_LOGF(LogReflectionCaptureComponent, Warning, "RefreshCapture(bSmoothBlend=true) on '%ls' will pop instead of cross-fading because r.ReflectionCapture.Runtime.SmoothBlendSlots is 0. Set it >0 (max %d) to enable smooth refresh blends.", *GetPathName(), 4);
		}
	}

	bRefreshRequested = true;
	bFastRenderRequested = bFastRender;
	bSmoothBlendRequested = bSmoothBlend;
}

FReflectionCaptureMapBuildData* UReflectionCaptureComponent::GetMapBuildData() const
{
	AActor* Owner = GetOwner();

	if (Owner)
	{
		ULevel* OwnerLevel = Owner->GetLevel();

		if (OwnerLevel && OwnerLevel->OwningWorld)
		{
			ULevel* ActiveLightingScenario = OwnerLevel->OwningWorld->GetActiveLightingScenario();
			UMapBuildDataRegistry* MapBuildData = NULL;

			if (ActiveLightingScenario && ActiveLightingScenario->MapBuildData)
			{
				MapBuildData = ActiveLightingScenario->MapBuildData;
			}
			else if (OwnerLevel->MapBuildData)
			{
				MapBuildData = OwnerLevel->MapBuildData;
			}			
			else if (OwnerLevel->OwningWorld->IsPartitionedWorld())
			{
				// Fallback to PersistentLevel in case of missing data in OwnerLevel
				// This is quite likely for WP maps in PIE/engine until the ReflectionsCaptures
				// are updated to distribute themselves in the cells (planned for 5.1)
				MapBuildData = OwnerLevel->OwningWorld->PersistentLevel->MapBuildData;
			}
			 
			if (MapBuildData)
			{
				FReflectionCaptureMapBuildData* ReflectionBuildData = MapBuildData->GetReflectionCaptureBuildData(MapBuildDataId);

				if (ReflectionBuildData && (ReflectionBuildData->CubemapSize == UReflectionCaptureComponent::GetReflectionCaptureSize() || ReflectionBuildData->HasBeenUploadedFinal()))
				{
					return ReflectionBuildData;
				}
			}
		}
	}
	
	return NULL;
}

bool UReflectionCaptureComponent::IsEncodedHDRCubemapTextureRequired(EShaderPlatform ShaderPlatform)
{
	const bool bEncodedHDRCubemapTextureRequired = (GIsEditor || IsMobilePlatform(ShaderPlatform))
		// mobile forward renderer or translucensy in mobile deferred need encoded reflection texture when clustered reflections disabled
		&& !MobileForwardEnableClusteredReflections(ShaderPlatform);
	return bEncodedHDRCubemapTextureRequired;
}

void UReflectionCaptureComponent::PropagateLightingScenarioChange()
{
	const FSceneInterface* Scene = GetWorld()->Scene;
	const EShaderPlatform  ShaderPlatform = Scene ? Scene->GetShaderPlatform() : GMaxRHIShaderPlatform;
	const bool bEncodedDataRequired = IsEncodedHDRCubemapTextureRequired(ShaderPlatform);

	if (bEncodedDataRequired && EncodedHDRCubemapTexture == nullptr)
	{
		ReregisterComponent();
	}

	if (!IsMobilePlatform(ShaderPlatform) || MobileForwardEnableClusteredReflections(ShaderPlatform))
	{
		// GetMapBuildData has changed, re-upload
		MarkDirtyForRecaptureOrUpload();
	}
}

AReflectionCapture::AReflectionCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CaptureComponent = CreateDefaultSubobject<UReflectionCaptureComponent>(TEXT("NewReflectionComponent"));

	bCanBeInCluster = true;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (!IsRunningCommandlet() && (SpriteComponent != nullptr))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			FName NAME_ReflectionCapture;
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> DecalTexture;
			FConstructorStatics()
				: NAME_ReflectionCapture(TEXT("ReflectionCapture"))
				, DecalTexture(TEXT("/Engine/EditorResources/S_ReflActorIcon"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.DecalTexture.Get();
		SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->bHiddenInGame = true;
		SpriteComponent->SetUsingAbsoluteScale(true);
		SpriteComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		SpriteComponent->bIsScreenSizeScaled = true;
	}

	CaptureOffsetComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("CaptureOffset"));
	if (!IsRunningCommandlet() && (CaptureOffsetComponent != nullptr))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			FName NAME_ReflectionCapture;
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> DecalTexture;
			FConstructorStatics()
				: NAME_ReflectionCapture(TEXT("ReflectionCapture"))
				, DecalTexture(TEXT("/Engine/EditorResources/S_ReflActorIcon"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		CaptureOffsetComponent->Sprite = ConstructorStatics.DecalTexture.Get();
		CaptureOffsetComponent->SetRelativeScale3D_Direct(FVector(0.2f, 0.2f, 0.2f));
		CaptureOffsetComponent->bHiddenInGame = true;
		CaptureOffsetComponent->SetUsingAbsoluteScale(true);
		CaptureOffsetComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		CaptureOffsetComponent->bIsScreenSizeScaled = true;
	}

	if (CaptureComponent)
	{
		CaptureComponent->CaptureOffsetComponent = CaptureOffsetComponent;
	}
	
#endif // WITH_EDITORONLY_DATA
}

ASphereReflectionCapture::ASphereReflectionCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<USphereReflectionCaptureComponent>(TEXT("NewReflectionComponent")))
{
	USphereReflectionCaptureComponent* SphereComponent = CastChecked<USphereReflectionCaptureComponent>(GetCaptureComponent());
	RootComponent = SphereComponent;
#if WITH_EDITORONLY_DATA
	if (GetSpriteComponent())
	{
		GetSpriteComponent()->SetupAttachment(SphereComponent);
	}
	if (GetCaptureOffsetComponent())
	{
		GetCaptureOffsetComponent()->SetupAttachment(SphereComponent);
	}
#endif	//WITH_EDITORONLY_DATA

	UDrawSphereComponent* DrawInfluenceRadius = CreateDefaultSubobject<UDrawSphereComponent>(TEXT("DrawRadius0"));
	DrawInfluenceRadius->SetupAttachment(GetCaptureComponent());
	DrawInfluenceRadius->bDrawOnlyIfSelected = true;
	DrawInfluenceRadius->bUseEditorCompositing = true;
	DrawInfluenceRadius->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SphereComponent->PreviewInfluenceRadius = DrawInfluenceRadius;

	DrawCaptureRadius = CreateDefaultSubobject<UDrawSphereComponent>(TEXT("DrawRadius1"));
	DrawCaptureRadius->SetupAttachment(GetCaptureComponent());
	DrawCaptureRadius->bDrawOnlyIfSelected = true;
	DrawCaptureRadius->bUseEditorCompositing = true;
	DrawCaptureRadius->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	DrawCaptureRadius->ShapeColor = FColor(100, 90, 40);
}

#if WITH_EDITOR
void ASphereReflectionCapture::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	USphereReflectionCaptureComponent* SphereComponent = Cast<USphereReflectionCaptureComponent>(GetCaptureComponent());
	check(SphereComponent);
	const FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 5000.0f : 50.0f );
	FMath::ApplyScaleToFloat(SphereComponent->InfluenceRadius, ModifiedScale);
	GetCaptureComponent()->InvalidateLightingCache();
	PostEditChange();
}

void APlaneReflectionCapture::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	UPlaneReflectionCaptureComponent* PlaneComponent = Cast<UPlaneReflectionCaptureComponent>(GetCaptureComponent());
	check(PlaneComponent);
	const FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 5000.0f : 50.0f );
	FMath::ApplyScaleToFloat(PlaneComponent->InfluenceRadiusScale, ModifiedScale);
	GetCaptureComponent()->InvalidateLightingCache();
	PostEditChange();
}
#endif

ABoxReflectionCapture::ABoxReflectionCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UBoxReflectionCaptureComponent>(TEXT("NewReflectionComponent")))
{
	UBoxReflectionCaptureComponent* BoxComponent = CastChecked<UBoxReflectionCaptureComponent>(GetCaptureComponent());
	BoxComponent->SetRelativeScale3D_Direct(FVector(1000, 1000, 400));
	RootComponent = BoxComponent;
#if WITH_EDITORONLY_DATA
	if (GetSpriteComponent())
	{
		GetSpriteComponent()->SetupAttachment(BoxComponent);


		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			FName NAME_ReflectionCapture;
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> Texture;
			FConstructorStatics()
				: NAME_ReflectionCapture(TEXT("ReflectionCapture"))
				, Texture(TEXT("/Engine/EditorResources/S_BoxReflectionCapture"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		GetSpriteComponent()->Sprite = ConstructorStatics.Texture.Get();
	}

	if (GetCaptureOffsetComponent())
	{
		GetCaptureOffsetComponent()->SetupAttachment(BoxComponent);
	}

#endif	//WITH_EDITORONLY_DATA
	UBoxComponent* DrawInfluenceBox = CreateDefaultSubobject<UBoxComponent>(TEXT("DrawBox0"));
	DrawInfluenceBox->SetupAttachment(GetCaptureComponent());
	DrawInfluenceBox->bDrawOnlyIfSelected = true;
	DrawInfluenceBox->bUseEditorCompositing = true;
	DrawInfluenceBox->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	DrawInfluenceBox->InitBoxExtent(FVector(1, 1, 1));
	BoxComponent->PreviewInfluenceBox = DrawInfluenceBox;

	UBoxComponent* DrawCaptureBox = CreateDefaultSubobject<UBoxComponent>(TEXT("DrawBox1"));
	DrawCaptureBox->SetupAttachment(GetCaptureComponent());
	DrawCaptureBox->bDrawOnlyIfSelected = true;
	DrawCaptureBox->bUseEditorCompositing = true;
	DrawCaptureBox->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	DrawCaptureBox->ShapeColor = FColor(100, 90, 40);
	DrawCaptureBox->InitBoxExtent(FVector(1, 1, 1));
	BoxComponent->PreviewCaptureBox = DrawCaptureBox;
}

APlaneReflectionCapture::APlaneReflectionCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UPlaneReflectionCaptureComponent>(TEXT("NewReflectionComponent")))
{
	UPlaneReflectionCaptureComponent* PlaneComponent = CastChecked<UPlaneReflectionCaptureComponent>(GetCaptureComponent());
	PlaneComponent->SetRelativeScale3D_Direct(FVector(1, 1000, 1000));
	RootComponent = PlaneComponent;
#if WITH_EDITORONLY_DATA
	if (GetSpriteComponent())
	{
		GetSpriteComponent()->SetupAttachment(PlaneComponent);
	}
	if (GetCaptureOffsetComponent())
	{
		GetCaptureOffsetComponent()->SetupAttachment(PlaneComponent);
	}
#endif	//#if WITH_EDITORONLY_DATA
	UDrawSphereComponent* DrawInfluenceRadius = CreateDefaultSubobject<UDrawSphereComponent>(TEXT("DrawRadius0"));
	DrawInfluenceRadius->SetupAttachment(GetCaptureComponent());
	DrawInfluenceRadius->bDrawOnlyIfSelected = true;
	DrawInfluenceRadius->SetUsingAbsoluteScale(true);
	DrawInfluenceRadius->bUseEditorCompositing = true;
	DrawInfluenceRadius->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	PlaneComponent->PreviewInfluenceRadius = DrawInfluenceRadius;

	UBoxComponent* DrawCaptureBox = CreateDefaultSubobject<UBoxComponent>(TEXT("DrawBox1"));
	DrawCaptureBox->SetupAttachment(GetCaptureComponent());
	DrawCaptureBox->bDrawOnlyIfSelected = true;
	DrawCaptureBox->bUseEditorCompositing = true;
	DrawCaptureBox->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	DrawCaptureBox->ShapeColor = FColor(100, 90, 40);
	DrawCaptureBox->InitBoxExtent(FVector(1, 1, 1));
	PlaneComponent->PreviewCaptureBox = DrawCaptureBox;
}

FColor RGBMEncode( FLinearColor Color, float MaxValueRGBM)
{
	FColor Encoded;

	// Convert to gamma space
	Color.R = FMath::Sqrt( Color.R );
	Color.G = FMath::Sqrt( Color.G );
	Color.B = FMath::Sqrt( Color.B );

	// Range
	Color /= MaxValueRGBM;
	
	float MaxValue = FMath::Max( FMath::Max(Color.R, Color.G), FMath::Max(Color.B, UE_DELTA) );
	
	if( MaxValue > 0.75f )
	{
		// Fit to valid range by leveling off intensity
		float Tonemapped = ( MaxValue - 0.75 * 0.75 ) / ( MaxValue - 0.5 );
		Color *= Tonemapped / MaxValue;
		MaxValue = Tonemapped;
	}

	Encoded.A = FMath::Min( FMath::CeilToInt( MaxValue * 255.0f ), 255 );
	Encoded.R = FMath::RoundToInt( ( Color.R * 255.0f / Encoded.A ) * 255.0f );
	Encoded.G = FMath::RoundToInt( ( Color.G * 255.0f / Encoded.A ) * 255.0f );
	Encoded.B = FMath::RoundToInt( ( Color.B * 255.0f / Encoded.A ) * 255.0f );

	return Encoded;
}

// Based off of CubemapGen
// https://code.google.com/p/cubemapgen/

#define FACE_X_POS 0
#define FACE_X_NEG 1
#define FACE_Y_POS 2
#define FACE_Y_NEG 3
#define FACE_Z_POS 4
#define FACE_Z_NEG 5

#define EDGE_LEFT   0	 // u = 0
#define EDGE_RIGHT  1	 // u = 1
#define EDGE_TOP    2	 // v = 0
#define EDGE_BOTTOM 3	 // v = 1

#define CORNER_NNN  0
#define CORNER_NNP  1
#define CORNER_NPN  2
#define CORNER_NPP  3
#define CORNER_PNN  4
#define CORNER_PNP  5
#define CORNER_PPN  6
#define CORNER_PPP  7

// D3D cube map face specification
//   mapping from 3D x,y,z cube map lookup coordinates 
//   to 2D within face u,v coordinates
//
//   --------------------> U direction 
//   |                   (within-face texture space)
//   |         _____
//   |        |     |
//   |        | +Y  |
//   |   _____|_____|_____ _____
//   |  |     |     |     |     |
//   |  | -X  | +Z  | +X  | -Z  |
//   |  |_____|_____|_____|_____|
//   |        |     |
//   |        | -Y  |
//   |        |_____|
//   |
//   v   V direction
//      (within-face texture space)

// Index by [Edge][FaceOrEdge]
static const int32 CubeEdgeListA[12][2] =
{
	{FACE_X_POS, EDGE_LEFT},
	{FACE_X_POS, EDGE_RIGHT},
	{FACE_X_POS, EDGE_TOP},
	{FACE_X_POS, EDGE_BOTTOM},

	{FACE_X_NEG, EDGE_LEFT},
	{FACE_X_NEG, EDGE_RIGHT},
	{FACE_X_NEG, EDGE_TOP},
	{FACE_X_NEG, EDGE_BOTTOM},

	{FACE_Z_POS, EDGE_TOP},
	{FACE_Z_POS, EDGE_BOTTOM},
	{FACE_Z_NEG, EDGE_TOP},
	{FACE_Z_NEG, EDGE_BOTTOM}
};

static const int32 CubeEdgeListB[12][2] =
{
	{FACE_Z_POS, EDGE_RIGHT },
	{FACE_Z_NEG, EDGE_LEFT  },
	{FACE_Y_POS, EDGE_RIGHT },
	{FACE_Y_NEG, EDGE_RIGHT },

	{FACE_Z_NEG, EDGE_RIGHT },
	{FACE_Z_POS, EDGE_LEFT  },
	{FACE_Y_POS, EDGE_LEFT  },
	{FACE_Y_NEG, EDGE_LEFT  },

	{FACE_Y_POS, EDGE_BOTTOM },
	{FACE_Y_NEG, EDGE_TOP    },
	{FACE_Y_POS, EDGE_TOP    },
	{FACE_Y_NEG, EDGE_BOTTOM },
};

// Index by [Face][Corner]
static const int32 CubeCornerList[6][4] =
{
	{ CORNER_PPP, CORNER_PPN, CORNER_PNP, CORNER_PNN },
	{ CORNER_NPN, CORNER_NPP, CORNER_NNN, CORNER_NNP },
	{ CORNER_NPN, CORNER_PPN, CORNER_NPP, CORNER_PPP },
	{ CORNER_NNP, CORNER_PNP, CORNER_NNN, CORNER_PNN },
	{ CORNER_NPP, CORNER_PPP, CORNER_NNP, CORNER_PNP },
	{ CORNER_PPN, CORNER_NPN, CORNER_PNN, CORNER_NNN }
};

static void EdgeWalkSetup( bool ReverseDirection, int32 Edge, int32 MipSize, int32& EdgeStart, int32& EdgeStep )
{
	if( ReverseDirection )
	{
		switch( Edge )
		{
		case EDGE_LEFT:		//start at lower left and walk up
			EdgeStart = MipSize * (MipSize - 1);
			EdgeStep = -MipSize;
			break;
		case EDGE_RIGHT:	//start at lower right and walk up
			EdgeStart = MipSize * (MipSize - 1) + (MipSize - 1);
			EdgeStep = -MipSize;
			break;
		case EDGE_TOP:		//start at upper right and walk left
			EdgeStart = (MipSize - 1);
			EdgeStep = -1;
			break;
		case EDGE_BOTTOM:	//start at lower right and walk left
			EdgeStart = MipSize * (MipSize - 1) + (MipSize - 1);
			EdgeStep = -1;
			break;
		}            
	}
	else
	{
		switch( Edge )
		{
		case EDGE_LEFT:		//start at upper left and walk down
			EdgeStart = 0;
			EdgeStep = MipSize;
			break;
		case EDGE_RIGHT:	//start at upper right and walk down
			EdgeStart = (MipSize - 1);
			EdgeStep = MipSize;
			break;
		case EDGE_TOP:		//start at upper left and walk left
			EdgeStart = 0;
			EdgeStep = 1;
			break;
		case EDGE_BOTTOM:	//start at lower left and walk left
			EdgeStart = MipSize * (MipSize - 1);
			EdgeStep = 1;
			break;
		}
	}
}

void GenerateEncodedHDRData(const TArray<uint8>& FullHDRData, int32 CubemapSize, TArray<uint8>& OutEncodedHDRData)
{
	check(FullHDRData.Num() > 0);
	const int32 NumMips = FMath::CeilLogTwo(CubemapSize) + 1;

	int32 SourceMipBaseIndex = 0;
	int32 DestMipBaseIndex = 0;

	int32 EncodedDataSize = FullHDRData.Num() * sizeof(FFloat3Packed) / sizeof(FFloat16Color);

	OutEncodedHDRData.Empty(EncodedDataSize);
	OutEncodedHDRData.AddZeroed(EncodedDataSize);

	for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
	{
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const int32 SourceCubeFaceBytes = MipSize * MipSize * sizeof(FFloat16Color);
		const int32 DestCubeFaceBytes = MipSize * MipSize * sizeof(FFloat3Packed);

		const FFloat16Color*	MipSrcData = (const FFloat16Color*)&FullHDRData[SourceMipBaseIndex];
		FFloat3Packed*			MipDstData = (FFloat3Packed*)&OutEncodedHDRData[DestMipBaseIndex];

		// Encode rest of texels
		for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
		{
			const int32 FaceSourceIndex = SourceMipBaseIndex + CubeFace * SourceCubeFaceBytes;
			const int32 FaceDestIndex = DestMipBaseIndex + CubeFace * DestCubeFaceBytes;
			const FFloat16Color* FaceSourceData = (const FFloat16Color*)&FullHDRData[FaceSourceIndex];
			FFloat3Packed* FaceDestData = (FFloat3Packed*)&OutEncodedHDRData[FaceDestIndex];

			// Convert each texel from linear space FP16 to RG11B10 FColor

			// Find MaxValue
			for (int32 y = 0; y < MipSize; y++)
			{
				for (int32 x = 0; x < MipSize; x++)
				{
					int32 TexelIndex = x + y * MipSize;
					const FLinearColor LinearColor = FLinearColor(FaceSourceData[TexelIndex]);
					FaceDestData[ TexelIndex ] = FFloat3Packed(LinearColor);
				}
			}
		}

		SourceMipBaseIndex += SourceCubeFaceBytes * CubeFace_MAX;
		DestMipBaseIndex += DestCubeFaceBytes * CubeFace_MAX;
	}
}

/**
 * A cubemap texture resource that knows how to upload the packed capture data from a reflection capture.
 * @todo - support texture streaming and compression
 */
class FReflectionTextureCubeResource : public FTexture
{
public:

	FReflectionTextureCubeResource() :
		Size(0),
		NumMips(0),
		Format(PF_Unknown)
	{}

	void SetupParameters(int32 InSize, int32 InNumMips, EPixelFormat InFormat, TArray<uint8>&& InSourceData)
	{
		Size = InSize;
		NumMips = InNumMips;
		Format = InFormat;
		SourceData = MoveTemp(InSourceData);
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::CreateCube(TEXT("ReflectionTextureCube"), Size, Format)
			.SetNumMips(NumMips)
			.SetFlags(ETextureCreateFlags::ShaderResource);

		TextureCubeRHI = RHICmdList.CreateTexture(Desc);
		TextureRHI = TextureCubeRHI;

		if (SourceData.Num())
		{
			const int32 BlockBytes = GPixelFormats[Format].BlockBytes;
			int32 MipBaseIndex = 0;

			for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
			{
				const int32 MipSize = 1 << (NumMips - MipIndex - 1);
				const int32 CubeFaceBytes = MipSize * MipSize * BlockBytes;

				uint32 SrcPitch = MipSize * BlockBytes;

				for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
				{
					uint32 DestPitch = 0;
					uint8* DestBuffer = (uint8*)RHICmdList.GetAsImmediate().LockTextureCubeFace(TextureCubeRHI, CubeFace, 0, MipIndex, RLM_WriteOnly, DestPitch, false);
					
					const int32 SourceIndex = MipBaseIndex + CubeFace * CubeFaceBytes;
					const uint8* SourceBuffer = &SourceData[SourceIndex];

					if (SrcPitch == DestPitch)
					{
						FMemory::Memcpy(DestBuffer, SourceBuffer, CubeFaceBytes);
					}
					else
					{
						// Copy data, taking the stride into account!
						uint8* Src = (uint8*)SourceBuffer;
						uint8* Dst = (uint8*)DestBuffer;
						for (int32 Row = 0; Row < MipSize; ++Row)
						{
							FMemory::Memcpy(Dst, Src, SrcPitch);
							Src += SrcPitch;
							Dst += DestPitch;
						}
						check((PTRINT(Src) - PTRINT(SourceBuffer)) == PTRINT(CubeFaceBytes));
					}

					RHICmdList.GetAsImmediate().UnlockTextureCubeFace(TextureCubeRHI, CubeFace, 0, MipIndex, false);
				}

				MipBaseIndex += CubeFaceBytes * CubeFace_MAX;
			}

			SourceData.Empty();
		}

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer
		(
			SF_Trilinear,
			AM_Clamp,
			AM_Clamp,
			AM_Clamp
		);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		INC_MEMORY_STAT_BY(STAT_ReflectionCaptureTextureMemory, CalcTextureSize(Size, Size, Format, NumMips) * 6);
	}

	virtual void ReleaseRHI() override
	{
		DEC_MEMORY_STAT_BY(STAT_ReflectionCaptureTextureMemory, CalcTextureSize(Size, Size, Format, NumMips) * 6);
		TextureCubeRHI.SafeRelease();
		FTexture::ReleaseRHI();
	}

	virtual uint32 GetSizeX() const override
	{
		return Size;
	}

	virtual uint32 GetSizeY() const override //-V524
	{
		return Size;
	}

	FRHITexture* GetTextureRHI()
	{
		return TextureCubeRHI;
	}

private:

	int32 Size;
	int32 NumMips;
	EPixelFormat Format;
	FTextureRHIRef TextureCubeRHI;

	TArray<uint8> SourceData;
};

TArray<UReflectionCaptureComponent*> UReflectionCaptureComponent::ReflectionCapturesToUpdate;
TArray<UReflectionCaptureComponent*> UReflectionCaptureComponent::ReflectionCapturesToUpdateForLoad;
FCriticalSection UReflectionCaptureComponent::ReflectionCapturesToUpdateForLoadLock;

UReflectionCaptureComponent::UReflectionCaptureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Brightness = 1;
	// Shouldn't be able to change reflection captures at runtime
	Mobility = EComponentMobility::Static;
	EncodedHDRCubemapTexture = nullptr;
	CachedAverageBrightness = 1.0f;
	bNeedsRecaptureOrUpload = false;
	bRuntimeCapture = false;
	RuntimeLastRenderedFrame = 0;
	RuntimeSkylightScale = FLinearColor(0.1f, 0.1f, 0.1f);
}

void UReflectionCaptureComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	UpdatePreviewShape();

	if (ShouldComponentAddToScene() && ShouldRender())
	{
		// Mutually-exclusive runtime/baked modes (r.ReflectionCapture.Runtime): only add a proxy when this capture's type
		// matches the current mode.  Plane reflection captures are a separate system and are always added.
		const bool bRuntimeMode = CVarReflectionCaptureRuntime.GetValueOnGameThread() != 0;
		if (IsA<UPlaneReflectionCaptureComponent>() || (bRuntimeMode == IsRuntimeCapture()))
		{
			GetWorld()->Scene->AddReflectionCapture(this);
		}
	}
}

void UReflectionCaptureComponent::SendRenderTransform_Concurrent()
{	
	// Don't update the transform of a component that needs to be recaptured,
	// Otherwise the RT will get the new transform one frame before the capture
	if (!bNeedsRecaptureOrUpload)
	{
		UpdatePreviewShape();

		if (ShouldComponentAddToScene() && ShouldRender())
		{
			GetWorld()->Scene->UpdateReflectionCaptureTransform(this);
		}
	}

	Super::SendRenderTransform_Concurrent();
}

void UReflectionCaptureComponent::SafeReleaseEncodedHDRCubemapTexture()
{
	if (EncodedHDRCubemapTexture)
	{
		BeginReleaseResource(EncodedHDRCubemapTexture);
		ENQUEUE_RENDER_COMMAND(DeleteEncodedHDRCubemapTexture)(
			[ParamPointerToRelease = EncodedHDRCubemapTexture](FRHICommandListImmediate& RHICmdList)
			{
				delete ParamPointerToRelease;
			});
		EncodedHDRCubemapTexture = nullptr;
	}
}

float UReflectionCaptureComponent::ComputeCurrentRuntimeCaptureFade(double Now, float Duration) const
{
	if (RuntimeFade.StartValue == RuntimeFade.TargetValue)
	{
		return RuntimeFade.TargetValue;
	}
	if (Duration <= 0.0f)
	{
		return RuntimeFade.TargetValue;
	}
	const float Elapsed = static_cast<float>(Now - RuntimeFade.StartTime);
	const float Delta = Elapsed / Duration;
	const float Direction = (RuntimeFade.TargetValue >= RuntimeFade.StartValue) ? 1.0f : -1.0f;
	const float Value = RuntimeFade.StartValue + Direction * Delta;
	return FMath::Clamp(Value,
		FMath::Min(RuntimeFade.StartValue, RuntimeFade.TargetValue),
		FMath::Max(RuntimeFade.StartValue, RuntimeFade.TargetValue));
}

void UReflectionCaptureComponent::RuntimeCaptureFadeIn(double Now, float FromValue, TArray<FRuntimeFadeEntry>& FadeEntries)
{
	RuntimeFade.StartValue = FromValue;
	RuntimeFade.TargetValue = 1.0f;
	RuntimeFade.StartTime = Now;
	FadeEntries.Add({ RuntimeFade, this });
}

void UReflectionCaptureComponent::RuntimeCaptureFadeOut(double Now, float FromValue, TArray<FRuntimeFadeEntry>& FadeEntries)
{
	RuntimeFade.StartValue = FromValue;
	RuntimeFade.TargetValue = 0.0f;
	RuntimeFade.StartTime = Now;
	FadeEntries.Add({ RuntimeFade, this });
}

/** Reset to a steady state at Value (no fade). */
void UReflectionCaptureComponent::RuntimeCaptureFadeSet(float Value, TArray<FRuntimeFadeEntry>& FadeEntries)
{
	RuntimeFade.StartValue = Value;
	RuntimeFade.TargetValue = Value;
	RuntimeFade.StartTime = 0.0;
	FadeEntries.Add({ RuntimeFade, this });
}

void UReflectionCaptureComponent::AllocateEncodedHDRCubemapTexture(FStaticShaderPlatform ShaderPlatform)
{
	check(IsEncodedHDRCubemapTextureRequired(ShaderPlatform));
	
	FReflectionCaptureMapBuildData* MapBuildData = GetMapBuildData();
	// If the MapBuildData is valid, update it. If it is not we will use the cached values, if there are any
	if (MapBuildData && MapBuildData->CubemapSize > 0)
	{
		if (!EncodedHDRCubemapTexture)
		{
			// Make sure data is not discarded yet
			check(!MapBuildData->HasBeenUploadedFinal());
			
			EncodedHDRCubemapTexture = new FReflectionTextureCubeResource();
			TArray<uint8> EncodedHDRCapturedData;
			EPixelFormat EncodedHDRCubemapTextureFormat = PF_Unknown;
			if (IsMobileDeferredShadingEnabled(ShaderPlatform))
			{
				// make a copy, FullHDR data will still be needed later
				EncodedHDRCapturedData = MapBuildData->FullHDRCapturedData;
				EncodedHDRCubemapTextureFormat = PF_FloatRGBA;
			}
			else
			{
				if (GIsEditor)
				{
					EncodedHDRCapturedData = MapBuildData->EncodedHDRCapturedData;
				}
				else
				{
					// EncodedHDR data is no longer needed
					Swap(EncodedHDRCapturedData, MapBuildData->EncodedHDRCapturedData);
				}
				EncodedHDRCubemapTextureFormat = PF_FloatR11G11B10;
			}

			EncodedHDRCubemapTexture->SetupParameters(MapBuildData->CubemapSize, FMath::CeilLogTwo(MapBuildData->CubemapSize) + 1, EncodedHDRCubemapTextureFormat, MoveTemp(EncodedHDRCapturedData));
			BeginInitResource(EncodedHDRCubemapTexture);
		}

		CachedAverageBrightness = MapBuildData->AverageBrightness;
	}
}

void UReflectionCaptureComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();
	GetWorld()->Scene->RemoveReflectionCapture(this);
}

void UReflectionCaptureComponent::InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly)
{
	// Save the static mesh state for transactions, force it to be marked dirty if we are going to discard any static lighting data.
	Modify(true);

	Super::InvalidateLightingCacheDetailed(bInvalidateBuildEnqueuedLighting, bTranslationOnly);

	MapBuildDataId = FGuid::NewGuid();

	MarkRenderStateDirty();
}

void UReflectionCaptureComponent::PostInitProperties()
{
	Super::PostInitProperties();

	// Gets overwritten with saved value (if being loaded from disk)
	if(!IsTemplate())
	{
		FPlatformMisc::CreateGuid(MapBuildDataId);
	}
#if WITH_EDITOR
	bMapBuildDataIdLoaded = false;
#endif

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		FScopeLock Lock(&ReflectionCapturesToUpdateForLoadLock);
		ReflectionCapturesToUpdateForLoad.AddUnique(this);
		bNeedsRecaptureOrUpload = true; 
	}
}

void UReflectionCaptureComponent::SerializeLegacyData(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FReflectionCaptureObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.CustomVer(FReflectionCaptureObjectVersion::GUID) < FReflectionCaptureObjectVersion::MoveReflectionCaptureDataToMapBuildData)
	{
		if (Ar.UEVer() >= VER_UE4_REFLECTION_CAPTURE_COOKING)
		{
			bool bLegacy = false;
			Ar << bLegacy;
		}

		if (Ar.UEVer() >= VER_UE4_REFLECTION_DATA_IN_PACKAGES)
		{
			FGuid SavedVersion;
			Ar << SavedVersion;

			float AverageBrightness = 1.0f;

			if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::ReflectionCapturesStoreAverageBrightness)
			{
				Ar << AverageBrightness;
			}

			int32 EndOffset = 0;
			Ar << EndOffset;

			FGuid LegacyReflectionCaptureVer(0x0c669396, 0x9cb849ae, 0x9f4120ff, 0x5812f4d3);

			if (SavedVersion != LegacyReflectionCaptureVer)
			{
				// Guid version of saved source data doesn't match latest, skip the data
				// The skipping is done so we don't have to maintain legacy serialization code paths when changing the format
				Ar.Seek(EndOffset);
			}
			else
			{
				bool bValid = false;
				Ar << bValid;

				if (bValid)
				{
					FReflectionCaptureMapBuildData* LegacyMapBuildData = new FReflectionCaptureMapBuildData();

					if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::CustomReflectionCaptureResolutionSupport)
					{
						Ar << LegacyMapBuildData->CubemapSize;
					}
					else
					{
						LegacyMapBuildData->CubemapSize = 128;
					}

					{
						TArray<uint8> CompressedCapturedData;
						Ar << CompressedCapturedData;

						check(CompressedCapturedData.Num() > 0);
						FMemoryReader MemoryAr(CompressedCapturedData);

						int32 UncompressedSize;
						MemoryAr << UncompressedSize;

						int32 CompressedSize;
						MemoryAr << CompressedSize;

						LegacyMapBuildData->FullHDRCapturedData.Empty(UncompressedSize);
						LegacyMapBuildData->FullHDRCapturedData.AddUninitialized(UncompressedSize);

						const uint8* SourceData = &CompressedCapturedData[MemoryAr.Tell()];
						verify(FCompression::UncompressMemory(NAME_Zlib, LegacyMapBuildData->FullHDRCapturedData.GetData(), UncompressedSize, SourceData, CompressedSize));
					}

					LegacyMapBuildData->AverageBrightness = AverageBrightness;

					FReflectionCaptureMapBuildLegacyData LegacyComponentData;
					LegacyComponentData.Id = MapBuildDataId;
					LegacyComponentData.MapBuildData = LegacyMapBuildData;
					GReflectionCapturesWithLegacyBuildData.AddAnnotation(this, MoveTemp(LegacyComponentData));
				}
			}
		}
	}
}

void UReflectionCaptureComponent::Serialize(FArchive& Ar)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UReflectionCaptureComponent::Serialize"), STAT_ReflectionCaptureComponent_Serialize, STATGROUP_LoadTime);

#if WITH_EDITOR
	FGuid OldMapBuildDataId = MapBuildDataId;
#endif

	Super::Serialize(Ar);

	SerializeLegacyData(Ar);

#if WITH_EDITOR
	// Check to see if we overwrote the MapBuildDataId with a loaded one
	if (Ar.IsLoading())
	{
		bMapBuildDataIdLoaded = OldMapBuildDataId != MapBuildDataId;
	}
	else
	{
		// If we're cooking, display a deterministic cook warning if we didn't overwrite the generated GUID at load time
		UE_CLOGF(Ar.IsCooking() && !GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn) && !bMapBuildDataIdLoaded, LogReflectionCaptureComponent, Warning, "%ls contains a legacy UReflectionCaptureComponent and is being non-deterministically cooked - please resave the asset and recook.", *GetOutermost()->GetName());
	}
#endif
}

FReflectionCaptureProxy* UReflectionCaptureComponent::CreateSceneProxy()
{
	return new FReflectionCaptureProxy(this);
}

void UReflectionCaptureComponent::UpdatePreviewShape() 
{
	if (CaptureOffsetComponent)
	{
		CaptureOffsetComponent->SetRelativeLocation_Direct(CaptureOffset / GetComponentTransform().GetScale3D());
		if (CaptureOffset.IsNearlyZero())
		{
			CaptureOffsetComponent->SetVisibility(false);
		}
		else
		{
			CaptureOffsetComponent->SetVisibility(true);
		}
	}
}

#if WITH_EDITOR
bool UReflectionCaptureComponent::CanEditChange(const FProperty* Property) const
{
	bool bCanEditChange = Super::CanEditChange(Property);

	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UReflectionCaptureComponent, Cubemap) ||
		Property->GetFName() == GET_MEMBER_NAME_CHECKED(UReflectionCaptureComponent, SourceCubemapAngle))
	{
		bCanEditChange &= ReflectionSourceType == EReflectionSourceType::SpecifiedCubemap;
	}

	return bCanEditChange;
}

void UReflectionCaptureComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UReflectionCaptureComponent, Cubemap) ||
		PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UReflectionCaptureComponent, SourceCubemapAngle) ||
		PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UReflectionCaptureComponent, ReflectionSourceType))
	{
		MarkDirtyForRecapture();
	}

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UReflectionCaptureComponent, bRuntimeCapture))
	{
		ReflectionCaptureRuntimeModeChanged(this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

void UReflectionCaptureComponent::BeginDestroy()
{
	// Deregister the component from the update queue
	if (bNeedsRecaptureOrUpload)
	{
		FScopeLock Lock(&ReflectionCapturesToUpdateForLoadLock);
		ReflectionCapturesToUpdate.Remove(this);
		ReflectionCapturesToUpdateForLoad.Remove(this);
	}

	// Have to do this because we can't use GetWorld in BeginDestroy
	for (TSet<FSceneInterface*>::TConstIterator SceneIt(GetRendererModule().GetAllocatedScenes()); SceneIt; ++SceneIt)
	{
		FSceneInterface* Scene = *SceneIt;
		Scene->ReleaseReflectionCubemap(this);
	}

	if (EncodedHDRCubemapTexture)
	{
		BeginReleaseResource(EncodedHDRCubemapTexture);
	}

	// Begin a fence to track the progress of the above BeginReleaseResource being completed on the RT
	ReleaseResourcesFence.BeginFence();

	Super::BeginDestroy();
}

bool UReflectionCaptureComponent::IsReadyForFinishDestroy()
{
	// Wait until the fence is complete before allowing destruction
	return Super::IsReadyForFinishDestroy() && ReleaseResourcesFence.IsFenceComplete();
}

void UReflectionCaptureComponent::FinishDestroy()
{
	if (EncodedHDRCubemapTexture)
	{
		delete EncodedHDRCubemapTexture;
		EncodedHDRCubemapTexture = nullptr;
	}

	Super::FinishDestroy();
}

void UReflectionCaptureComponent::MarkDirtyForRecaptureOrUpload() 
{ 
	if (GetVisibleFlag())
	{
		ReflectionCapturesToUpdate.AddUnique(this);
		bNeedsRecaptureOrUpload = true; 
	}
}

void UReflectionCaptureComponent::MarkDirtyForRecapture() 
{ 
	if (GetVisibleFlag())
	{
		MarkPackageDirty();
		MapBuildDataId = FGuid::NewGuid();
		ReflectionCapturesToUpdate.AddUnique(this);
		bNeedsRecaptureOrUpload = true; 
	}
}

void UReflectionCaptureComponent::UpdateReflectionCaptureContents(UWorld* WorldToUpdate, const TCHAR* CaptureReason, bool bVerifyOnlyCapturing, bool bCapturingForMobile, bool bInsideTick)
{
	if (WorldToUpdate && WorldToUpdate->Scene
		// Don't capture and read back capture contents if we are currently doing async shader compiling
		// This will keep the update requests in the queue until compiling finishes
		// Note: this will also prevent uploads of cubemaps from DDC, which is unintentional
		&& (GShaderCompilingManager == NULL || !GShaderCompilingManager->IsCompiling())
#if WITH_EDITOR
		// Prevent any reflection capture if textures are still compiling
		&& FTextureCompilingManager::Get().GetNumRemainingTextures() == 0
#endif
		)
	{
		// If no captures to update or load, on the condition that SendAllEndOfFrameUpdates() does not change their status, 
		// WorldCombinedCaptures is empty and AllocateReflectionCaptures() simply returns. The logic here becomes simply calling
		// an unnecessary SendAllEndOfFrameUpdates() and return. So we could have this early out.
		
		const bool bHasCapturesToUpdate = ReflectionCapturesToUpdate.Num() > 0;
		const bool bHasCapturesToLoad = ReflectionCapturesToUpdateForLoad.Num() > 0;
		const bool bAlwaysUpdatesCaptures = CVarReflectionCaptureUpdateEveryFrame.GetValueOnGameThread() != 0;
		const bool bNeedsAnyUpdate = bHasCapturesToUpdate || bHasCapturesToLoad || bAlwaysUpdatesCaptures;

		// If runtime captures are enabled, "AllocateReflectionCaptures" is always called during world tick, to update timesliced runtime captures.
		int32 ReflectionCaptureRuntime = CVarReflectionCaptureRuntime.GetValueOnGameThread();

		// Track the previous value of the CVar.  If the previous frame had runtime captures enabled, but the current does not, we still need to call
		// AllocateReflectionCaptures one last time to clean up any active runtime captures.  Because we only call that during world tick, only update
		// ReflectionCaptureRuntimeLast during a world tick as well.
		static int32 ReflectionCaptureRuntimeLast = 0;
		bool bPreviousFrameRuntimeCaptures = ReflectionCaptureRuntimeLast != 0;
		if (bInsideTick)
		{
			ReflectionCaptureRuntimeLast = ReflectionCaptureRuntime;
		}

		if (!bNeedsAnyUpdate)
		{
			if ((ReflectionCaptureRuntime != 0 || bPreviousFrameRuntimeCaptures) && bInsideTick)
			{
				WorldToUpdate->Scene->AllocateReflectionCaptures({}, CaptureReason, bVerifyOnlyCapturing, bCapturingForMobile, bInsideTick);
			}
			return;
		}

		//guarantee that all render proxies are up to date before kicking off this render
		WorldToUpdate->SendAllEndOfFrameUpdates();

		int32 UpdateEveryFrameMode = CVarReflectionCaptureUpdateEveryFrame.GetValueOnGameThread();
		if (UpdateEveryFrameMode)
		{
			ForEachObjectOfClass(UReflectionCaptureComponent::StaticClass(), [WorldToUpdate, UpdateEveryFrameMode](UObject* ComponentObject)
			{
				if (ComponentObject->GetWorld() == WorldToUpdate)
				{
					if (UReflectionCaptureComponent* Component = CastChecked<UReflectionCaptureComponent>(ComponentObject))
					{
						AActor* Owner = Component->GetOwner();

#if WITH_EDITOR
						// UpdateEveryFrameMode == 2 indicates only selected objects should be refreshed, mode 1 refreshes everything.
						if (Owner && Component->IsVisibleInEditor() && !Owner->IsHiddenEd() && (UpdateEveryFrameMode == 1 || Component->IsSelectedInEditor() || Owner->IsSelectedInEditor()))
#endif
						{
							Component->MarkDirtyForRecapture();		// Continuously refresh reflection captures
						}
					}
				}
			}, /*bIncludeDerivedClasses=*/ true, RF_ClassDefaultObject, EInternalObjectFlags::Garbage);
		}

		TArray<UReflectionCaptureComponent*> WorldCombinedCaptures;

		for (int32 CaptureIndex = ReflectionCapturesToUpdate.Num() - 1; CaptureIndex >= 0; CaptureIndex--)
		{
			UReflectionCaptureComponent* CaptureComponent = ReflectionCapturesToUpdate[CaptureIndex];

			if (CaptureComponent->GetWorld() == WorldToUpdate)
			{
				// Captures without a SceneProxy are runtime captures subject to budgeting, which are handled in the
				// budgeting code path in FScene::AllocateReflectionCaptures.  Or captures disabled by the current
				// r.ReflectionCapture.Runtime mode.
				if (CaptureComponent->SceneProxy)
				{
					WorldCombinedCaptures.Add(CaptureComponent);
				}
				ReflectionCapturesToUpdate.RemoveAt(CaptureIndex);
			}
		}

		if (ReflectionCapturesToUpdateForLoad.Num() > 0)
		{
			FScopeLock Lock(&ReflectionCapturesToUpdateForLoadLock);
			for (int32 CaptureIndex = ReflectionCapturesToUpdateForLoad.Num() - 1; CaptureIndex >= 0; CaptureIndex--)
			{
				UReflectionCaptureComponent* CaptureComponent = ReflectionCapturesToUpdateForLoad[CaptureIndex];

				if (CaptureComponent->GetWorld() == WorldToUpdate)
				{
					if (CaptureComponent->SceneProxy)
					{
						WorldCombinedCaptures.Add(CaptureComponent);
					}
					ReflectionCapturesToUpdateForLoad.RemoveAt(CaptureIndex);
				}
			}
		}

		WorldToUpdate->Scene->AllocateReflectionCaptures(WorldCombinedCaptures, CaptureReason, bVerifyOnlyCapturing, bCapturingForMobile, bInsideTick);
	}
}

bool UReflectionCaptureComponent::HasReflectionCapturesToUpdate()
{
	return !ReflectionCapturesToUpdate.IsEmpty() || !ReflectionCapturesToUpdateForLoad.IsEmpty() || (CVarReflectionCaptureUpdateEveryFrame.GetValueOnGameThread() != 0);
}

#if WITH_EDITOR
void UReflectionCaptureComponent::PreFeatureLevelChange(ERHIFeatureLevel::Type PendingFeatureLevel)
{
	if (SupportsTextureCubeArray(PendingFeatureLevel))
	{
		SafeReleaseEncodedHDRCubemapTexture();

		MarkDirtyForRecaptureOrUpload();
	}
}
#endif // WITH_EDITOR

USphereReflectionCaptureComponent::USphereReflectionCaptureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InfluenceRadius = 3000;
}

void USphereReflectionCaptureComponent::UpdatePreviewShape()
{
	if (PreviewInfluenceRadius)
	{
		PreviewInfluenceRadius->InitSphereRadius(InfluenceRadius);
	}

	Super::UpdatePreviewShape();
}

#if WITH_EDITOR
void USphereReflectionCaptureComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// AActor::PostEditChange will ForceUpdateComponents()
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && 
		(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USphereReflectionCaptureComponent, InfluenceRadius)))
	{
		InvalidateLightingCache();
	}
}
#endif // WITH_EDITOR

UBoxReflectionCaptureComponent::UBoxReflectionCaptureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BoxTransitionDistance = 100;
}

void UBoxReflectionCaptureComponent::UpdatePreviewShape()
{
	if (PreviewCaptureBox)
	{
		PreviewCaptureBox->InitBoxExtent(((GetComponentTransform().GetScale3D() - FVector(BoxTransitionDistance)) / GetComponentTransform().GetScale3D()).ComponentMax(FVector::ZeroVector));
	}

	Super::UpdatePreviewShape();
}

float UBoxReflectionCaptureComponent::GetInfluenceBoundingRadius() const
{
	return GetComponentTransform().GetScale3D().Size();
}

#if WITH_EDITOR
void UBoxReflectionCaptureComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// AActor::PostEditChange will ForceUpdateComponents()
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBoxReflectionCaptureComponent, BoxTransitionDistance))
	{
		InvalidateLightingCache();
	}
}
#endif // WITH_EDITOR

UPlaneReflectionCaptureComponent::UPlaneReflectionCaptureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InfluenceRadiusScale = 2;
}

void UPlaneReflectionCaptureComponent::UpdatePreviewShape()
{
	if (PreviewInfluenceRadius)
	{
		PreviewInfluenceRadius->InitSphereRadius(GetInfluenceBoundingRadius());
	}

	Super::UpdatePreviewShape();
}

float UPlaneReflectionCaptureComponent::GetInfluenceBoundingRadius() const
{
	return FVector2D(GetComponentTransform().GetScale3D().Y, GetComponentTransform().GetScale3D().Z).Size() * InfluenceRadiusScale;
}

FReflectionCaptureProxy::FReflectionCaptureProxy(const UReflectionCaptureComponent* InComponent)
{
	PackedIndex = INDEX_NONE;
	SortedCaptureIndex = INDEX_NONE;
	CaptureOffset = FVector3f(InComponent->CaptureOffset);

	const USphereReflectionCaptureComponent* SphereComponent = Cast<const USphereReflectionCaptureComponent>(InComponent);
	const UBoxReflectionCaptureComponent* BoxComponent = Cast<const UBoxReflectionCaptureComponent>(InComponent);
	const UPlaneReflectionCaptureComponent* PlaneComponent = Cast<const UPlaneReflectionCaptureComponent>(InComponent);

	// Initialize shape specific settings
	Shape = EReflectionCaptureShape::Num;
	BoxTransitionDistance = 0;

	if (SphereComponent)
	{
		Shape = EReflectionCaptureShape::Sphere;
	}
	else if (BoxComponent)
	{
		Shape = EReflectionCaptureShape::Box;
		BoxTransitionDistance = BoxComponent->BoxTransitionDistance;
	}
	else if (PlaneComponent)
	{
		Shape = EReflectionCaptureShape::Plane;
	}
	else
	{
		check(0);
	}
	
	// Initialize common settings
	Component = InComponent;
	const FReflectionCaptureMapBuildData* MapBuildData = InComponent->GetMapBuildData();

	EncodedHDRCubemap = Component->EncodedHDRCubemapTexture;
	EncodedHDRAverageBrightness = Component->CachedAverageBrightness;
	SetTransform(InComponent->GetComponentTransform().ToMatrixWithScale());
	InfluenceRadius = InComponent->GetInfluenceBoundingRadius();
	Brightness = InComponent->Brightness;

	FString PathName = Component->GetPathName();
	Guid = FXxHash64::HashBuffer(*PathName, PathName.NumBytesWithoutNull());

	bUsingPreviewCaptureData = MapBuildData == NULL;
}

void FReflectionCaptureProxy::SetTransform(const FMatrix& InTransform)
{
	Position = FDFVector3(InTransform.GetOrigin());

	const FMatrix44f LocalToRelativeWorld = FMatrix44f(InTransform.RemoveTranslation());
	BoxTransform = LocalToRelativeWorld.Inverse();

	FVector3f ForwardVector(1.0f,0.0f,0.0f);
	FVector3f RightVector(0.0f,-1.0f,0.0f);
	const FVector4f PlaneNormal = LocalToRelativeWorld.TransformVector(ForwardVector);

	// Normalize the plane
	LocalReflectionPlane = FPlane4f(FVector3f::ZeroVector, FVector3f(PlaneNormal).GetSafeNormal());
	const FVector3f ReflectionXAxis = LocalToRelativeWorld.TransformVector(RightVector);
	const FVector3f ScaleVector = LocalToRelativeWorld.GetScaleVector();
	BoxScales = ScaleVector;
	// Include the owner's draw scale in the axes
	ReflectionXAxisAndYScale = FVector4(FVector4f(ReflectionXAxis.GetSafeNormal() * ScaleVector.Y));
	ReflectionXAxisAndYScale.W = ScaleVector.Y / ScaleVector.Z;
}

void FReflectionCaptureProxy::UpdateMobileUniformBuffer()
{
	UpdateMobileUniformBuffer(FRHICommandListImmediate::Get());
}

void FReflectionCaptureProxy::UpdateMobileUniformBuffer(FRHICommandListBase& RHICmdList)
{
	FTexture* CaptureTexture = GBlackTextureCube;
	if (EncodedHDRCubemap)
	{
		check(EncodedHDRCubemap->IsInitialized());
		CaptureTexture = EncodedHDRCubemap;
	}
		
	FMobileReflectionCaptureShaderParameters Parameters;
	//To keep ImageBasedReflectionLighting coherence with PC, use AverageBrightness instead of InvAverageBrightness to calculate the IBL contribution
	Parameters.Params = FVector4f(EncodedHDRAverageBrightness, 0.f, 0.0f, Brightness);
	Parameters.Texture = CaptureTexture->TextureRHI;
	Parameters.TextureSampler = CaptureTexture->SamplerStateRHI;
	Parameters.TextureBlend = Parameters.Texture;
	Parameters.TextureBlendSampler = Parameters.TextureSampler;

	if (MobileUniformBuffer.GetReference())
	{
		MobileUniformBuffer.UpdateUniformBufferImmediate(RHICmdList, Parameters);
	}
	else
	{
		MobileUniformBuffer = TUniformBufferRef<FMobileReflectionCaptureShaderParameters>::CreateUniformBufferImmediate(Parameters, UniformBuffer_MultiFrame);
	}
}

