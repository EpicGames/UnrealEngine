// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/RotatingMovementComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RotatingMovementComponent)

URotatingMovementComponent::URotatingMovementComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Rotating components are often added to actors also with a projectile component,
	// and we wouldn't want to stomp their desired physics volume setting so turn off auto-registration.
	bAutoRegisterPhysicsVolumeUpdates = false;
	bComponentShouldUpdatePhysicsVolume = false;

	RotationRate.Yaw = 180.0f;
	bRotationInLocalSpace = true;
}


void URotatingMovementComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	// skip if we don't want component updated when not rendered or if updated component can't move
	if (ShouldSkipUpdate(DeltaTime))
	{
		return;
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (!IsValid(UpdatedComponent))
	{
		return;
	}

	// Compute new rotation
	const FQuat OldRotation = UpdatedComponent->GetComponentQuat();
	const FQuat DeltaRotation = (RotationRate * DeltaTime).Quaternion();
	const FQuat NewRotation = bRotationInLocalSpace ? (OldRotation * DeltaRotation) : (DeltaRotation * OldRotation);

	// Compute new location
	FVector DeltaLocation = FVector::ZeroVector;
	if (!PivotTranslation.IsZero())
	{
		const FVector OldPivot = OldRotation.RotateVector(PivotTranslation);
		const FVector NewPivot = NewRotation.RotateVector(PivotTranslation);
		DeltaLocation = (OldPivot - NewPivot); // ConstrainDirectionToPlane() not necessary because it's done by MoveUpdatedComponent() below.
	}

	const bool bEnableCollision = false;
	MoveUpdatedComponent(DeltaLocation, NewRotation, bEnableCollision);
}

