// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RuntimeVirtualTextureComponent.h"

#include "Async/TaskGraphInterfaces.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "GameDelegates.h"
#include "GameFramework/Actor.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "RenderUtils.h"
#include "RHIGlobals.h"
#include "SceneInterface.h"
#include "SceneUtils.h"
#include "UnrealEngine.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/VirtualTexture.h"
#include "VT/VirtualTextureBuilder.h"
#include "VT/VirtualTextureBuiltData.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimeVirtualTextureComponent)

#define LOCTEXT_NAMESPACE "URuntimeVirtualTextureComponent"

static TAutoConsoleVariable<bool> CVarVTStreamingMips(
	TEXT("r.VT.RVT.StreamingMips"),
	true,
	TEXT("Enable streaming mips for RVT"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)	{ FGlobalComponentRecreateRenderStateContext Context; }),
	ECVF_Default);

#if WITH_EDITOR

static TAutoConsoleVariable<int32> CVarVTStreamingMipsShowInEditor(
	TEXT("r.VT.RVT.StreamingMips.UseInEditor"),
	1,
	TEXT("Use streaming mips for RVT when in Editor.\n")
	TEXT("  0: Never use.\n")
	TEXT("  1: Use the setting from RVT component (default).\n")
	TEXT("  2: Always use when available.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) { FGlobalComponentRecreateRenderStateContext Context; }),
	ECVF_Default);

#endif

static TAutoConsoleVariable<bool> CVarVTStreamingMipsUseAlways(
	TEXT("r.VT.RVT.StreamingMips.UseAlways"),
	false,
	TEXT("Whenever streaming low mips are in use, only show the streaming mips and never show runtime generated pages.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) { FGlobalComponentRecreateRenderStateContext Context; }),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarVTVisLogManualInvalidations(
	TEXT("r.VT.RVT.VisLogManualInvalidations"),
	false,
	TEXT("Outputs the RVT component invalidations and their bounds in the Visual Logger"),
	ECVF_Default);


URuntimeVirtualTextureComponent::URuntimeVirtualTextureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EnableInGamePerPlatform(true)
	, SceneProxy(nullptr)
{
	Mobility = EComponentMobility::Stationary;
}

void URuntimeVirtualTextureComponent::BeginDestroy()
{
	Super::BeginDestroy();
	
	// Queuing up a render fence means that we will have cleaned up the scene proxy/virtual texture producer before finishing the destroy.
	// This means that any transcode tasks will have finished *before* we garbage collect our StreamingTexture.
	// That's important because the transcode tasks reference the FVirtualTextureBuiltData from the StreamingTexture.
	DestroyFence.BeginFence();
}

bool URuntimeVirtualTextureComponent::IsReadyForFinishDestroy()
{
	bool bResult = Super::IsReadyForFinishDestroy() && DestroyFence.IsFenceComplete();
	return bResult;
}

bool URuntimeVirtualTextureComponent::IsActiveInWorld() const
{
	UWorld* World = GetWorld();
	return ((World != nullptr) 
		&& ((World->WorldType == EWorldType::Game) 
			|| (World->WorldType == EWorldType::Editor) 
			|| (World->WorldType == EWorldType::PIE)));
}

void URuntimeVirtualTextureComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AddedRuntimeVirtualTextureUseStreamingMipsInEditorMode)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UseStreamingMipsInEditorMode = bUseStreamingMipsInEditor_DEPRECATED ? ERuntimeVirtualTextureUseStreamingMipsInEditorMode::Always : ERuntimeVirtualTextureUseStreamingMipsInEditorMode::Never;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

#if WITH_EDITOR

void URuntimeVirtualTextureComponent::OnRegister()
{
	Super::OnRegister();

	// PIE duplicate will take ownership of the URuntimeVirtualTexture, so we add a delegate to be called when PIE finishes allowing us to retake ownership.
	PieEndDelegateHandle = FGameDelegates::Get().GetEndPlayMapDelegate().AddUObject(this, &URuntimeVirtualTextureComponent::MarkRenderStateDirty);
}

void URuntimeVirtualTextureComponent::OnUnregister()
{
	FGameDelegates::Get().GetEndPlayMapDelegate().Remove(PieEndDelegateHandle);
	PieEndDelegateHandle.Reset();

	Super::OnUnregister();
}

#endif

void URuntimeVirtualTextureComponent::SetVirtualTexture(URuntimeVirtualTexture* InVirtualTexture)
{
	VirtualTexture = InVirtualTexture;
	MarkRenderStateDirty();
}

void URuntimeVirtualTextureComponent::GetHidePrimitiveSettings(bool& OutHidePrimitiveEditor, bool& OutHidePrimitiveGame) const
{
	OutHidePrimitiveEditor = bHidePrimitives;
	OutHidePrimitiveGame = bHidePrimitives;
	// Evaluate the bound delegates (who we expect to OR in their settings).
	HidePrimitivesDelegate.Broadcast(OutHidePrimitiveEditor, OutHidePrimitiveGame);
}

bool URuntimeVirtualTextureComponent::ShouldCreateRenderState() const
{
	// Make sure to have the component do nothing if VT is disabled or if the world is not compatible with RVT
	return Super::ShouldCreateRenderState() && IsActiveInWorld() && UseVirtualTexturing(GetScene()->GetShaderPlatform());
}

void URuntimeVirtualTextureComponent::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);
	// Mark transform as dirty after a world origin rebase. See comment in SendRenderTransform_Concurrent() below.
	MarkRenderTransformDirty();
}

void URuntimeVirtualTextureComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	if (VirtualTexture != nullptr)
	{
		// This will modify the URuntimeVirtualTexture and allocate its VT
		GetScene()->AddRuntimeVirtualTexture(this);
	}

	Super::CreateRenderState_Concurrent(Context);
}

void URuntimeVirtualTextureComponent::SendRenderTransform_Concurrent()
{
	if (IsRenderStateCreated())
	{
		// We do a full recreate of the URuntimeVirtualTexture here which can cause a visual glitch.
		// We do this because, for an arbitrary transform, there is no way to only modify the transform and maintain the VT contents.
		// Possibly, with some work, the contents could be maintained for any transform change that is an exact multiple of the page size in world space.
		GetScene()->AddRuntimeVirtualTexture(this);
	}

	Super::SendRenderTransform_Concurrent();
}

void URuntimeVirtualTextureComponent::DestroyRenderState_Concurrent()
{
	// This will modify the URuntimeVirtualTexture and free its VT
	GetScene()->RemoveRuntimeVirtualTexture(this);

	Super::DestroyRenderState_Concurrent();
}

static ERuntimeVirtualTextureMaterialQuality ConvertMaterialQualityEnum(EMaterialQualityLevel::Type InMaterialQualityLevel)
{
	switch (InMaterialQualityLevel)
	{
	case EMaterialQualityLevel::Low: return ERuntimeVirtualTextureMaterialQuality::Low;
	case EMaterialQualityLevel::Medium: return ERuntimeVirtualTextureMaterialQuality::Medium;
	case EMaterialQualityLevel::High: return ERuntimeVirtualTextureMaterialQuality::High;
	case EMaterialQualityLevel::Epic: return ERuntimeVirtualTextureMaterialQuality::Epic;
	default: check(0);
	}

	return ERuntimeVirtualTextureMaterialQuality::Low;
}

bool URuntimeVirtualTextureComponent::IsEnabledInScene() const
{
	const EShaderPlatform ShaderPlatform = GetScene()->GetShaderPlatform();
	const bool bUseNanite = UseNanite(ShaderPlatform);
	if (bEnableForNaniteOnly && !bUseNanite)
	{
		return false;
	}

	if (!RuntimeVirtualTexture::IsMaterialTypeSupported(VirtualTexture->GetMaterialType(), ShaderPlatform))
	{
		return false;
	}

	if (UWorld* World = GetWorld())
	{
		if (World->WorldType == EWorldType::Game || World->WorldType == EWorldType::PIE)
		{
			if (!EnableInGamePerPlatform.GetValue())
			{
				return false;
			}

			const ERuntimeVirtualTextureMaterialQuality CurrentQuality = ConvertMaterialQualityEnum(GetCachedScalabilityCVars().MaterialQualityLevel);

			if (IsMobilePlatform(ShaderPlatform) && bUseMinMaterialQualityMobile)
			{
				if (CurrentQuality < MinInGameMaterialQualityMobile)
				{
					return false;
				}
			}
			else if (bUseMinMaterialQuality)
			{
 				if (CurrentQuality < MinInGameMaterialQuality)
 				{
 					return false;
 				}
			}
		}
	}

	return true;
}

void URuntimeVirtualTextureComponent::Invalidate(FBoxSphereBounds const& InWorldBounds, EVTInvalidatePriority InvalidatePriority)
{
	if (GetScene() != nullptr)
	{
		GetScene()->InvalidateRuntimeVirtualTexture(this, InWorldBounds, InvalidatePriority);

		UE_CVLOG_BOX(CVarVTVisLogManualInvalidations.GetValueOnGameThread(), this, LogVirtualTexturing, Log, InWorldBounds.GetBox(), (InvalidatePriority == EVTInvalidatePriority::High) ? FColor::Red : FColor::Green,
			TEXT("Invalidate RVT for %s(%s) %s"), *GetOwner()->GetActorNameOrLabel(), VirtualTexture ? *VirtualTexture->GetName() : TEXT_EMPTY, (InvalidatePriority == EVTInvalidatePriority::High) ? TEXT("(prioritized)"): TEXT_EMPTY);
	}
}

void URuntimeVirtualTextureComponent::RequestPreload(FBoxSphereBounds const& InWorldBounds, int32 InLevel)
{
	if (GetScene() != nullptr)
	{
		GetScene()->RequestPreloadRuntimeVirtualTexture(this, InWorldBounds, InLevel);
	}
}

FBoxSphereBounds URuntimeVirtualTextureComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FBox(FVector(0.f, 0.f, 0.f), FVector(1.f, 1.f, 1.f))).TransformBy(LocalToWorld);
}

#if WITH_EDITOR
void URuntimeVirtualTextureComponent::SetBoundsAlignActor(AActor* InActor)
{
	BoundsAlignActor = InActor;
}
#endif

FTransform URuntimeVirtualTextureComponent::GetTexelSnapTransform() const
{
	FVector Offset(ForceInitToZero);
	if (bSnapBoundsToLandscape && VirtualTexture != nullptr)
	{
		Offset = GetRelativeScale3D() * -0.5f / VirtualTexture->GetSize();
		Offset.Z = 0.f;
	}
	return FTransform(Offset);
}

// If a setting change can cause the streaming texture to no longer be valid then it should be included in this hash.
union FPackedStreamingMipsSettings
{
	uint64 PackedValue;
	struct
	{
		uint32 PackedSettingsVersion : 4;
		uint32 MaterialType : 4;
		uint32 TileSize : 12;
		uint32 TileBorderSize : 4;
		uint32 LODGroup : 8;
		uint32 CompressTextures : 1;
		uint32 SinglePhysicalSpace : 1;
		uint32 ContinuousUpdate_Deprecated : 1;
		uint32 bUseLowQualityCompression : 1;
		uint32 LossyCompressionAmount : 4;
	};

	static uint64 RemoveDeprecated(uint64 InHash)
	{
		// Remove deprecated parts of the the packed build hash.
		// Need to do this at runtime to handle old data.
		FPackedStreamingMipsSettings Settings;
		Settings.PackedValue = InHash;
		Settings.ContinuousUpdate_Deprecated = 0;
		return Settings.PackedValue;
	}
};

uint64 URuntimeVirtualTextureComponent::CalculateStreamingTextureSettingsHash() const
{
	// Shouldn't need to call this when there is VirtualTexture == nullptr
	check(VirtualTexture != nullptr);

	FPackedStreamingMipsSettings Settings;
	Settings.PackedValue = 0;
	Settings.PackedSettingsVersion = 2;
	Settings.MaterialType = (uint32)VirtualTexture->GetMaterialType();
	Settings.TileSize = (uint32)VirtualTexture->GetTileSize();
	Settings.TileBorderSize = (uint32)VirtualTexture->GetTileBorderSize();
	Settings.LODGroup = (uint32)VirtualTexture->GetLODGroup();
	Settings.CompressTextures = (uint32)VirtualTexture->GetCompressTextures();
	Settings.SinglePhysicalSpace = (uint32)VirtualTexture->GetSinglePhysicalSpace();
	Settings.bUseLowQualityCompression = (uint32)VirtualTexture->GetLQCompression();
	Settings.LossyCompressionAmount = (uint32)GetLossyCompressionAmount();

	return Settings.PackedValue;
}

bool URuntimeVirtualTextureComponent::IsStreamingLowMips(EShadingPath ShadingPath) const
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		const int32 ShowStreamingMipsInEditor = CVarVTStreamingMipsShowInEditor.GetValueOnAnyThread();
		if (ShowStreamingMipsInEditor == 0)
		{
			return false;
		}
		else if (ShowStreamingMipsInEditor == 1)
		{
			UWorld* World = GetWorld();
			if ((UseStreamingMipsInEditorMode == ERuntimeVirtualTextureUseStreamingMipsInEditorMode::Never)
				|| ((UseStreamingMipsInEditorMode == ERuntimeVirtualTextureUseStreamingMipsInEditorMode::PIEOnly) && ((World == nullptr) || !World->IsPlayInEditor())))
			{
				return false;
			}
		}
	}
#endif
	return VirtualTexture != nullptr && StreamingTexture != nullptr && StreamingTexture->GetVirtualTexture(ShadingPath) != nullptr && CVarVTStreamingMips.GetValueOnAnyThread();
}

bool URuntimeVirtualTextureComponent::IsStreamingLowMipsOnly()
{
	return bUseStreamingMipsOnly || CVarVTStreamingMipsUseAlways.GetValueOnAnyThread();
}

/** 
 * This test should be covered by the BuildHash check, but there was an bug where the texture compilation built the streaming virtual texture with an unexpected pixel format. 
 * The bug was fixed but keeping this extra check to catch any similar regression in future.
 */
static bool IsCompatibleFormat(URuntimeVirtualTexture const& RuntimeVirtualTexture, UVirtualTexture2D const& StreamingVirtualTexture)
{
	if (FTexturePlatformData const* StreamingTextureData = StreamingVirtualTexture.GetPlatformData())
	{
		if (FVirtualTextureBuiltData const* VTData = StreamingTextureData->VTData)
		{
			for (int32 LayerIndex = 0; LayerIndex < RuntimeVirtualTexture.GetLayerCount(); ++LayerIndex)
			{
				if (RuntimeVirtualTexture.GetLayerFormat(LayerIndex) != VTData->LayerTypes[LayerIndex])
				{
					return false;
				}
			}
			return true;
		}
	}
	return false;
}

bool URuntimeVirtualTextureComponent::IsStreamingTextureInvalid(EShadingPath ShadingPath) const
{
	EStreamingTextureStatusFlags Status = GetStreamingTextureStatus(ShadingPath);
	return EnumHasAllFlags(Status, EStreamingTextureStatusFlags::HasVirtualTexture | EStreamingTextureStatusFlags::HasStreamingTexture)
		&& EnumHasAnyFlags(Status, EStreamingTextureStatusFlags::InvalidStreamingTexture | EStreamingTextureStatusFlags::NonMatchingStreamingTextureSettings);
}

URuntimeVirtualTextureComponent::EStreamingTextureStatusFlags URuntimeVirtualTextureComponent::GetStreamingTextureStatus(EShadingPath ShadingPath) const
{
	EStreamingTextureStatusFlags Status = EStreamingTextureStatusFlags::None;
	if (VirtualTexture != nullptr)
	{
		Status |= EStreamingTextureStatusFlags::HasVirtualTexture;
	}
	if (StreamingTexture != nullptr)
	{
		Status |= EStreamingTextureStatusFlags::HasStreamingTexture;
	}

	if (EnumHasAllFlags(Status, EStreamingTextureStatusFlags::HasVirtualTexture | EStreamingTextureStatusFlags::HasStreamingTexture))
	{
		UVirtualTexture2D* BuiltTexture = ((ShadingPath == EShadingPath::Mobile) && StreamingTexture->bSeparateTextureForMobile) ? StreamingTexture->TextureMobile : StreamingTexture->Texture;
		if (BuiltTexture == nullptr)
		{
			Status |= EStreamingTextureStatusFlags::InvalidStreamingTexture;
		}
		else
		{
			const uint64 StreamingMipsBuildHash = FPackedStreamingMipsSettings::RemoveDeprecated(StreamingTexture->BuildHash);
			if (StreamingMipsBuildHash != CalculateStreamingTextureSettingsHash() || !IsCompatibleFormat(*VirtualTexture, *BuiltTexture))
			{
				Status |= EStreamingTextureStatusFlags::NonMatchingStreamingTextureSettings;
			}
		}
	}
	return Status;
}

EVTProducerPriority URuntimeVirtualTextureComponent::GetPriority() const
{
	return (VirtualTexture != nullptr) ? VirtualTexture->GetPriority() : EVTProducerPriority::Normal;
}

FVector4f URuntimeVirtualTextureComponent::GetCustomMaterialData() const
{
	return (VirtualTexture != nullptr) ? VirtualTexture->GetCustomMaterialData() : FVector4f::Zero();
}

#if WITH_EDITOR

bool URuntimeVirtualTextureComponent::IsStreamingTextureInvalid() const
{
	return IsStreamingTextureInvalid(EShadingPath::Mobile) || IsStreamingTextureInvalid(EShadingPath::Deferred);
}

URuntimeVirtualTextureComponent::EStreamingTextureStatusFlags URuntimeVirtualTextureComponent::GetStreamingTextureStatus() const
{
	return GetStreamingTextureStatus(EShadingPath::Deferred) | GetStreamingTextureStatus(EShadingPath::Mobile);
}

FLinearColor URuntimeVirtualTextureComponent::GetStreamingMipsFixedColor() const 
{
	if (!bUseStreamingMipsFixedColor)
	{
		return FLinearColor::Transparent;
	} 
	
	FLinearColor Color(StreamingMipsFixedColor);
	Color.A = 1.f;
	return Color;
}

static TextureCompressionSettings GetCompressionSettingFromLayerFormat(EPixelFormat LayerFormat)
{
	switch (LayerFormat)
	{
	case PF_BC5: return TC_Normalmap;
	case PF_BC4: return TC_Alpha;
	case PF_G16: return TC_Grayscale;
	}
	return TC_Default;
}

static void GetLayerFormatSettings(FTextureFormatSettings& OutFormatSettings, EPixelFormat LayerFormat, bool IsLayerYCoCg, bool IsLayerSRGB, bool IsLayerLQCompression)
{
	OutFormatSettings.CompressionSettings = IsLayerLQCompression ? TC_LQ : GetCompressionSettingFromLayerFormat(LayerFormat);
	OutFormatSettings.CompressionNone = LayerFormat == PF_B8G8R8A8 || LayerFormat == PF_G16;
	OutFormatSettings.CompressionNoAlpha = LayerFormat == PF_DXT1 || LayerFormat == PF_BC5 || LayerFormat == PF_R5G6B5_UNORM;
	OutFormatSettings.CompressionForceAlpha = LayerFormat == PF_DXT5;
	OutFormatSettings.CompressionYCoCg = IsLayerYCoCg;
	OutFormatSettings.SRGB = IsLayerSRGB;
}

void URuntimeVirtualTextureComponent::InitializeStreamingTexture(EShadingPath ShadingPath, uint32 InSizeX, uint32 InSizeY, uint8* InData)
{
	// We need an existing StreamingTexture object to update.
	if (IsActiveInWorld() && VirtualTexture != nullptr && StreamingTexture != nullptr)
	{
		FVirtualTextureBuildDesc BuildDesc;
		BuildDesc.bSinglePhysicalSpace = VirtualTexture->GetSinglePhysicalSpace();

		BuildDesc.TileSize = VirtualTexture->GetTileSize();
		BuildDesc.TileBorderSize = VirtualTexture->GetTileBorderSize();
		BuildDesc.LODGroup = VirtualTexture->GetLODGroup();
		BuildDesc.LossyCompressionAmount = GetLossyCompressionAmount();
		
		BuildDesc.LayerCount = VirtualTexture->GetLayerCount();
		check(BuildDesc.LayerCount <= RuntimeVirtualTexture::MaxTextureLayers);
		BuildDesc.LayerFormats.AddDefaulted(BuildDesc.LayerCount);
		BuildDesc.LayerFormatSettings.AddDefaulted(BuildDesc.LayerCount);

		for (int32 Layer = 0; Layer < BuildDesc.LayerCount; Layer++) 
		{
			const EPixelFormat LayerFormat = VirtualTexture->GetLayerFormat(Layer);
			BuildDesc.LayerFormats[Layer] = LayerFormat == PF_G16 || LayerFormat == PF_BC4 ? TSF_G16 : TSF_BGRA8;
			bool IsLayerLQCompression = (VirtualTexture->GetMaterialType() == ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness && VirtualTexture->GetLQCompression() && LayerFormat != PF_B8G8R8A8);
			GetLayerFormatSettings(BuildDesc.LayerFormatSettings[Layer], LayerFormat, VirtualTexture->IsLayerYCoCg(Layer), VirtualTexture->IsLayerSRGB(Layer), IsLayerLQCompression);
		}

		BuildDesc.BuildHash = CalculateStreamingTextureSettingsHash();

		BuildDesc.InSizeX = InSizeX;
		BuildDesc.InSizeY = InSizeY;
		BuildDesc.InData = InData;

		// Make sure the streaming texture is fully built before marking the render state dirty, otherwise the scene proxy will be constructed thinking that it's not, which will prevent showing it in editor. 
		//  It's a rarely-triggered, bake-time, editor-only function anyway, so the blocking wait is acceptable
		constexpr bool bWaitForCompilation = true;
		StreamingTexture->BuildTexture(ShadingPath, BuildDesc, bWaitForCompilation);
		StreamingTexture->Modify();
	}
}

bool URuntimeVirtualTextureComponent::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEdit = Super::CanEditChange(InProperty);
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URuntimeVirtualTextureComponent, UseStreamingMipsInEditorMode) ||
		InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URuntimeVirtualTextureComponent, bUseStreamingMipsOnly))
	{
		bCanEdit &= GetVirtualTexture() != nullptr && GetStreamingTexture() != nullptr;
	}
	return bCanEdit;
}

void URuntimeVirtualTextureComponent::CheckForErrors()
{
	Super::CheckForErrors();

	// Check if streaming texture has been built with the latest settings. If not then it won't be used which would cause a performance regression.
	if (IsActiveInWorld() && IsStreamingTextureInvalid())
	{
		FMessageLog("MapCheck").PerformanceWarning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("RuntimeVirtualTextureComponent_StreamingTextureNeedsUpdate", "The settings have changed since the streaming texture was last rebuilt. Streaming mips are disabled.")))
			->AddToken(FMapErrorToken::Create(FName("RuntimeVirtualTextureComponent_StreamingTextureNeedsUpdate")));
	}
}

#endif

#undef LOCTEXT_NAMESPACE

