// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	
=============================================================================*/

#include "Components/SceneCaptureComponent.h"
#include "Camera/CameraTypes.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "SceneInterface.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/Material.h"
#include "Components/BillboardComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "SceneManagement.h"
#include "Engine/StaticMesh.h"
#include "Engine/SceneCapture.h"
#include "Engine/SceneCapture2D.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Engine/SceneCaptureCube.h"
#include "Components/SceneCaptureComponentCube.h"
#include "SceneCapture/SceneCaptureInternal.h"
#include "Components/DrawFrustumComponent.h"
#include "Engine/PlanarReflection.h"
#include "Components/PlanarReflectionComponent.h"
#include "PlanarReflectionSceneProxy.h"
#include "Components/BoxComponent.h"
#include "Logging/MessageLog.h"
#if WITH_EDITOR
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#endif
#include "Engine/SCS_Node.h"
#include "Engine/TextureRenderTarget2D.h"
#include "UnrealEngine.h"
#include "SceneRenderBuilderInterface.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneCaptureComponent)

#define LOCTEXT_NAMESPACE "SceneCaptureComponent"

static TMultiMap<TWeakObjectPtr<UWorld>, TWeakObjectPtr<USceneCaptureComponent> > SceneCapturesToUpdateMap;
static FCriticalSection SceneCapturesToUpdateMapCS;

static TAutoConsoleVariable<bool> CVarSCOverrideOrthographicTilingValues(
	TEXT("r.SceneCapture.OverrideOrthographicTilingValues"),
	false,
	TEXT("Override defined orthographic values from SceneCaptureComponent2D - Ignored in Perspective mode."),
	ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarSCEnableOrthographicTiling(
	TEXT("r.SceneCapture.EnableOrthographicTiling"),
	false,
	TEXT("Render the scene in n frames (i.e TileCount) - Ignored in Perspective mode, works only in Orthographic mode and when r.SceneCapture.OverrideOrthographicTilingValues is on."),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSCOrthographicNumXTiles(
	TEXT("r.SceneCapture.OrthographicNumXTiles"),
	4,
	TEXT("Number of X tiles to render. Ignored in Perspective mode, works only in Orthographic mode and when r.SceneCapture.OverrideOrthographicTilingValues is on."),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSCOrthographicNumYTiles(
	TEXT("r.SceneCapture.OrthographicNumYTiles"),
	4,
	TEXT("Number of Y tiles to render. Ignored in Perspective mode, works only in Orthographic mode and when r.SceneCapture.OverrideOrthographicTilingValues is on."),
	ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarSCCullByDetailMode(
	TEXT("r.SceneCapture.CullByDetailMode"),
	1,
	TEXT("Whether to prevent scene capture updates according to the current detail mode"),
	ECVF_Scalability);

ASceneCapture::ASceneCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"));
	RootComponent = SceneComponent;
}

void ASceneCapture::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FEditorObjectVersion::GUID) < FEditorObjectVersion::ChangeSceneCaptureRootComponent)
	{
		if (IsTemplate())
		{
			if (UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass()))
			{
				for (USCS_Node* RootNode : BPClass->SimpleConstructionScript->GetRootNodes())
				{
					static const FName OldMeshName("CamMesh0");
					static const FName OldFrustumName("DrawFrust0");
					static const FName NewRootName("SceneComponent");
					if (RootNode->ParentComponentOrVariableName == OldMeshName || RootNode->ParentComponentOrVariableName == OldFrustumName)
					{
						RootNode->ParentComponentOrVariableName = NewRootName;
					}
				}
			}
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (MeshComp_DEPRECATED)
		{
			MeshComp_DEPRECATED->SetStaticMesh(nullptr);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
}

void ASceneCapture::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
}
// -----------------------------------------------

ASceneCapture2D::ASceneCapture2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CaptureComponent2D = CreateDefaultSubobject<USceneCaptureComponent2D>(TEXT("NewSceneCaptureComponent2D"));
	CaptureComponent2D->SetupAttachment(RootComponent);
}

void ASceneCapture2D::OnInterpToggle(bool bEnable)
{
	CaptureComponent2D->SetVisibility(bEnable);
}

void ASceneCapture2D::CalcCamera(float DeltaTime, FMinimalViewInfo& OutMinimalViewInfo)
{
	if (USceneCaptureComponent2D* SceneCaptureComponent = GetCaptureComponent2D())
	{
		SceneCaptureComponent->GetCameraView(DeltaTime, OutMinimalViewInfo);
	}
	else
	{
		Super::CalcCamera(DeltaTime, OutMinimalViewInfo);
	}
}
// -----------------------------------------------

ASceneCaptureCube::ASceneCaptureCube(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CaptureComponentCube = CreateDefaultSubobject<USceneCaptureComponentCube>(TEXT("NewSceneCaptureComponentCube"));
	CaptureComponentCube->SetupAttachment(RootComponent);
}

void ASceneCaptureCube::OnInterpToggle(bool bEnable)
{
	CaptureComponentCube->SetVisibility(bEnable);
}

// -----------------------------------------------
USceneCaptureComponent::USceneCaptureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), ShowFlags(FEngineShowFlags(ESFIM_Game))
{
	CaptureSource = SCS_SceneColorHDR;
	bCaptureEveryFrame = true;
	bCaptureOnMovement = true;
	bOverrideVirtualTextureThrottle = false;
	bCaptureGpuNextRender = false;
	bDumpGpuNextRender = false;
	bSuppressGpuCaptureOrDump = false;
	bAlwaysPersistRenderingState = false;
	LODDistanceFactor = 1.0f;
	MaxViewDistanceOverride = -1;
	CaptureSortPriority = 0;
	bUseRayTracingIfEnabled = 0;
	SkylightScale = FLinearColor::White;

	// Disable features that are not desired when capturing the scene
	ShowFlags.SetMotionBlur(0); // motion blur doesn't work correctly with scene captures.
	ShowFlags.SetSeparateTranslucency(0);
	ShowFlags.SetHMDDistortion(0);
	ShowFlags.SetOnScreenDebug(0);

	if (!IsTemplate())
	{
		if (HasAnyFlags(RF_NeedPostLoad) || GetOuter()->HasAnyFlags(RF_NeedPostLoad))
		{
			// Delegate registration is not thread-safe, so we postpone it on PostLoad when coming from loading which could be on another thread
		}
		else
		{
			RegisterDelegates();
		}
}
}

void USceneCaptureComponent::RegisterDelegates()
{
	ensureMsgf(IsInGameThread(), TEXT("Potential race condition in USceneCaptureComponent registering to a delegate from non game-thread"));
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddUObject(this, &USceneCaptureComponent::ReleaseGarbageReferences);
}

void USceneCaptureComponent::UnregisterDelegates()
{
	ensureMsgf(IsInGameThread(), TEXT("Potential race condition in USceneCaptureComponent unregistering to a delegate from non game-thread"));
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
}

void USceneCaptureComponent::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		RegisterDelegates();
	}

	UpdateShowFlags();
}

const TArray<FEngineShowFlagsSetting>& USceneCaptureComponent::GetShowFlagSettings() const
{
	return ShowFlagSettings;
}

void USceneCaptureComponent::SetShowFlagSettings(const TArray<FEngineShowFlagsSetting>& InShowFlagSettings)
{
	ShowFlagSettings = InShowFlagSettings;

	UpdateShowFlags();
}

void USceneCaptureComponent::BeginDestroy()
{
	Super::BeginDestroy();

	if (!IsTemplate())
	{
		UnregisterDelegates();
	}
}

// Because HiddenActors and ShowOnlyActors are serialized they are not easily refactored into weak pointers.
// Before GC runs, release pointers to any actors that have been explicitly marked garbage.
void USceneCaptureComponent::ReleaseGarbageReferences()
{
	// We only want to remove resolved actors that are explicitly marked as garbage, not unresolved pointers.
	auto Predicate = [](TObjectPtr<AActor> Ptr)
	{
		if (AActor* Actor = Ptr.Get())
		{
			return Actor->HasAnyInternalFlags(EInternalObjectFlags::Garbage);
		}
		return false;
	};
	HiddenActors.RemoveAll(Predicate);
	ShowOnlyActors.RemoveAll(Predicate);
}

void USceneCaptureComponent::OnRegister()
{
#if WITH_EDITORONLY_DATA
	// Check IsListedInSceneOutliner() to filter out hidden singleton runtime reflection capture component from showing up as an object in the editor.
	AActor* MyOwner = GetOwner();
	if ((MyOwner != nullptr) && !IsRunningCommandlet() && MyOwner->IsListedInSceneOutliner())
	{
		if (ProxyMeshComponent == nullptr)
		{
			ProxyMeshComponent = NewObject<UStaticMeshComponent>(MyOwner, NAME_None, RF_Transactional | RF_TextExportTransient);
			ProxyMeshComponent->SetupAttachment(this);
			ProxyMeshComponent->SetIsVisualizationComponent(true);
			ProxyMeshComponent->SetStaticMesh(Cast<UStaticMesh>(CaptureMeshPath.TryLoad()));
			ProxyMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			ProxyMeshComponent->bHiddenInGame = true;
			ProxyMeshComponent->CastShadow = false;
			ProxyMeshComponent->CreationMethod = CreationMethod;
			ProxyMeshComponent->RegisterComponentWithWorld(GetWorld());
		}
	}
#endif

	Super::OnRegister();

	// Make sure any loaded saved flag settings are reflected in our FEngineShowFlags
	UpdateShowFlags();
}

void USceneCaptureComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

#if WITH_EDITORONLY_DATA
	if (ProxyMeshComponent)
	{
		ProxyMeshComponent->DestroyComponent();
	}
#endif
}

void USceneCaptureComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	USceneCaptureComponent* This = CastChecked<USceneCaptureComponent>(InThis);

	for (int32 ViewIndex = 0; ViewIndex < This->ViewStates.Num(); ViewIndex++)
	{
		FSceneViewStateInterface* Ref = This->ViewStates[ViewIndex].GetReference();
		if (Ref)
		{
			Ref->AddReferencedObjects(Collector);
		}
	}

#if WITH_EDITORONLY_DATA
	Collector.AddReferencedObject(This->ProxyMeshComponent);
#endif
	Super::AddReferencedObjects(This, Collector);
}

void USceneCaptureComponent::HideComponent(UPrimitiveComponent* InComponent)
{
	if (InComponent)
	{
		TWeakObjectPtr<UPrimitiveComponent> WeakComponent(InComponent);
		HiddenComponents.AddUnique(WeakComponent);
	}
}

void USceneCaptureComponent::HideActorComponents(AActor* InActor, const bool bIncludeFromChildActors)
{
	if (InActor)
	{
		TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents(InActor, bIncludeFromChildActors);
		for (UPrimitiveComponent* PrimComp : PrimitiveComponents)
		{
			TWeakObjectPtr<UPrimitiveComponent> WeakComponent(PrimComp);
			HiddenComponents.AddUnique(WeakComponent);
		}
	}
}

void USceneCaptureComponent::ShowOnlyComponent(UPrimitiveComponent* InComponent)
{
	if (InComponent)
	{
		// Backward compatibility - set PrimitiveRenderMode to PRM_UseShowOnlyList if BP / game code tries to add a ShowOnlyComponent
		PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
		ShowOnlyComponents.AddUnique(InComponent);
	}
}

void USceneCaptureComponent::ShowOnlyActorComponents(AActor* InActor, const bool bIncludeFromChildActors)
{
	if (InActor)
	{
		// Backward compatibility - set PrimitiveRenderMode to PRM_UseShowOnlyList if BP / game code tries to add a ShowOnlyComponent
		PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;

		TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents(InActor, bIncludeFromChildActors);
		for (UPrimitiveComponent* PrimComp : PrimitiveComponents)
		{
			ShowOnlyComponents.AddUnique(PrimComp);
		}
	}
}

void USceneCaptureComponent::RemoveShowOnlyComponent(UPrimitiveComponent* InComponent)
{
	TWeakObjectPtr<UPrimitiveComponent> WeakComponent(InComponent);
	ShowOnlyComponents.Remove(WeakComponent);
}

void USceneCaptureComponent::RemoveShowOnlyActorComponents(AActor* InActor, const bool bIncludeFromChildActors)
{
	if (InActor)
	{
		TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents(InActor, bIncludeFromChildActors);
		for (UPrimitiveComponent* PrimComp : PrimitiveComponents)
		{
			TWeakObjectPtr<UPrimitiveComponent> WeakComponent(PrimComp);
			ShowOnlyComponents.Remove(WeakComponent);
		}
	}
}

void USceneCaptureComponent::ClearShowOnlyComponents()
{
	ShowOnlyComponents.Reset();
}

void USceneCaptureComponent::ClearHiddenComponents()
{
	HiddenComponents.Reset();
}

void USceneCaptureComponent::SetCaptureSortPriority(int32 NewCaptureSortPriority)
{
	CaptureSortPriority = NewCaptureSortPriority;
}

FSceneViewStateInterface* USceneCaptureComponent::GetViewState(int32 ViewIndex)
{
	while (ViewIndex >= ViewStates.Num())
	{
		ViewStates.Add(new FSceneViewStateReference());

		// Cube map view states can share an origin, saving memory and performance
		if ((ViewStates.Num() > 1) && IsCube())
		{
			ViewStates.Last().ShareOrigin(&ViewStates[0]);
		}
	}

	FSceneViewStateInterface* ViewStateInterface = ViewStates[ViewIndex].GetReference();
	if ((bCaptureEveryFrame || bAlwaysPersistRenderingState) && ViewStateInterface == NULL)
	{
		const ERHIFeatureLevel::Type FeatureLevel = GetScene() ? GetScene()->GetFeatureLevel() : GMaxRHIFeatureLevel;
		ViewStates[ViewIndex].Allocate(FeatureLevel);
		ViewStateInterface = ViewStates[ViewIndex].GetReference();
	}
	else if (!bCaptureEveryFrame && ViewStateInterface && !bAlwaysPersistRenderingState)
	{
		ViewStates[ViewIndex].Destroy();
		ViewStateInterface = NULL;
	}
	return ViewStateInterface;
}

void USceneCaptureComponent::UpdateShowFlags()
{
	USceneCaptureComponent* Archetype = Cast<USceneCaptureComponent>(GetArchetype());
	if (Archetype)
	{
		ShowFlags = Archetype->ShowFlags;
	}

	for (FEngineShowFlagsSetting ShowFlagSetting : ShowFlagSettings)
	{
		int32 SettingIndex = ShowFlags.FindIndexByName(*(ShowFlagSetting.ShowFlagName));
		if (SettingIndex != INDEX_NONE)
		{ 
			ShowFlags.SetSingleFlag(SettingIndex, ShowFlagSetting.Enabled);
		}
	}
}

#if WITH_EDITOR

bool USceneCaptureComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent, HiddenActors))
		{
			return PrimitiveRenderMode == ESceneCapturePrimitiveRenderMode::PRM_LegacySceneCapture ||
				PrimitiveRenderMode == ESceneCapturePrimitiveRenderMode::PRM_RenderScenePrimitives;
		}
		else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent, ShowOnlyActors))
		{
			return PrimitiveRenderMode == ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
		}
	}

	return true;
}

void USceneCaptureComponent::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	// If our ShowFlagSetting UStruct changed, (or if PostEditChange was called without specifying a property) update the actual show flags
	if (MemberPropertyName.IsEqual("ShowFlagSettings") || MemberPropertyName.IsNone())
	{
		UpdateShowFlags();
	}

	// The bSuppressGpuCaptureOrDump flag needs to be set to true when bCaptureGpuNextRender or bDumpGpuNextRender are changed, to suppress
	// GPU capture or dump for the automatic render following component re-registration.  These two properties are the only ones in the
	// component flagged as non-transactional (so they don't go in the undo/redo stream), so we can detect them based on that flag.
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->HasAnyPropertyFlags(CPF_NonTransactional))
	{
		bSuppressGpuCaptureOrDump = true;
	}
}
#endif

void USceneCaptureComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::AddedbUseShowOnlyList)
	{
		if (ShowOnlyActors.Num() > 0 || ShowOnlyComponents.Num() > 0)
		{
			PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
		}
	}
}

void USceneCaptureComponent::UpdateDeferredCaptures(FSceneInterface* Scene)
{
	TUniquePtr<ISceneRenderBuilder> SceneRenderBuilder = ISceneRenderBuilder::Create(Scene);
	SceneCaptureUpdateDeferredCapturesInternal(Scene, {}, *SceneRenderBuilder);
	SceneRenderBuilder->Execute();
}

void SceneCaptureUpdateDeferredCapturesInternal(FSceneInterface* Scene, TConstArrayView<const FSceneViewFamily*> ViewFamilies, ISceneRenderBuilder& SceneRenderBuilder)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SceneCaptureUpdateDeferredCapturesInternal);

	FScopeLock ScopeLock(&SceneCapturesToUpdateMapCS);
	UWorld* World = Scene->GetWorld();
	if (!World || SceneCapturesToUpdateMap.Num() == 0)
	{
		return;
	}

	// Only update the scene captures associated with the current scene.
	// Updating others not associated with the scene would cause invalid data to be rendered into the target
	TArray< TWeakObjectPtr<USceneCaptureComponent> > SceneCapturesToUpdate;
	SceneCapturesToUpdateMap.MultiFind(World, SceneCapturesToUpdate);

	const FSceneViewFamily* MainViewFamily = nullptr;
	for (const FSceneViewFamily* Family : ViewFamilies)
	{
		if (Family->bIsMainViewFamily)
		{
			MainViewFamily = Family;
			break;
		}
	}

	int32 IncrementIfNotRemoved;
	for (int32 CaptureIndex = 0; CaptureIndex < SceneCapturesToUpdate.Num(); CaptureIndex += IncrementIfNotRemoved)
	{
		IncrementIfNotRemoved = 1;

		if (SceneCapturesToUpdate[CaptureIndex].IsValid())
		{
			if (USceneCaptureComponent2D* Component = Cast<USceneCaptureComponent2D>(SceneCapturesToUpdate[CaptureIndex].Get()))
			{
				// If this is intended to render with the main view family, and we don't have a main view family,
				// remove it from the array so it can render with the main view family later.
				if (Component->ShouldRenderWithMainViewFamily() && !MainViewFamily)
				{
					SceneCapturesToUpdate.RemoveAtSwap(CaptureIndex);
					IncrementIfNotRemoved = 0;
				}
			}
		}
	}

	SceneCapturesToUpdate.Sort([](const TWeakObjectPtr<USceneCaptureComponent>& A, const TWeakObjectPtr<USceneCaptureComponent>& B)
	{
		if (!A.IsValid())
		{
			return false;
		}
		else if (!B.IsValid())
		{
			return true;
		}
		return A->CaptureSortPriority > B->CaptureSortPriority;
	});

	for (TWeakObjectPtr<USceneCaptureComponent> Component : SceneCapturesToUpdate)
	{
		if (Component.IsValid())
		{
			bool bIsCaptureComponent2D = Component->Is2D();
			if (bIsCaptureComponent2D)
			{
				((USceneCaptureComponent2D*)Component.Get())->MainViewFamily = MainViewFamily;
			}

			Component->UpdateSceneCaptureContents(Scene, SceneRenderBuilder);

			if (bIsCaptureComponent2D)
			{
				((USceneCaptureComponent2D*)Component.Get())->MainViewFamily = nullptr;
			}
		}
	}

	// All scene captures for this world have been updated
	SceneCapturesToUpdateMap.Remove(World);
}

void USceneCaptureComponent::OnUnregister()
{
	// Make sure this component isn't still in the update map before we fully unregister
	{
		FScopeLock ScopeLock(&SceneCapturesToUpdateMapCS);
		SceneCapturesToUpdateMap.Remove(GetWorld(), this);
	}
	for (int32 ViewIndex = 0; ViewIndex < ViewStates.Num(); ViewIndex++)
	{
		ViewStates[ViewIndex].Destroy();
	}

	// Manually destroy the view state array here.  To account for the possibility of "FSceneViewStateReference::ShareOrigin" being used,
	// where later view states reference the first item in the array, we delete the later items first.
	if (ViewStates.Num() > 1)
	{
		ViewStates.RemoveAt(1, ViewStates.Num() - 1);
	}
	ViewStates.Empty();

	Super::OnUnregister();
}

bool USceneCaptureComponent::IsCulledByDetailMode() const
{
	return CVarSCCullByDetailMode.GetValueOnAnyThread() && DetailMode > GetCachedScalabilityCVars().DetailMode;
}

// -----------------------------------------------


USceneCaptureComponent2D::USceneCaptureComponent2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FOVAngle = 90.0f;

	FirstPersonFieldOfView = 90.0f;
	FirstPersonScale = 1.0f;

	OrthoWidth = DEFAULT_ORTHOWIDTH;
	bAutoCalculateOrthoPlanes = true;
	AutoPlaneShift = 0.0f;
	bUpdateOrthoPlanes = false;
	bUseCameraHeightAsViewTarget = false;
	Overscan = 0.0;
	
	bUseCustomProjectionMatrix = false;
	bAutoActivate = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;

	// Tick in the editor so that bCaptureEveryFrame preview works
	bTickInEditor = true;

	// default to full blend weight..
	PostProcessBlendWeight = 1.0f;
	CustomProjectionMatrix.SetIdentity();
	ClipPlaneNormal = FVector(0, 0, 1);
	bCameraCutThisFrame = false;
	bEnableFirstPersonFieldOfView = false;
	bEnableFirstPersonScale = false;
	bConsiderUnrenderedOpaquePixelAsFullyTranslucent = false;

	bMainViewFamily = false;
	bMainViewResolution = false;
	bMainViewCamera = false;
	bInheritMainViewCameraPostProcessSettings = false;
	bIgnoreScreenPercentage = false;
	
	TileID = 0;

	// Update default post-processing values to reflect default values (previously and still) applied in code.
	PostProcessSettings.DynamicGlobalIlluminationMethod = EDynamicGlobalIlluminationMethod::None;
	PostProcessSettings.ReflectionMethod = EReflectionMethod::None;
	PostProcessSettings.LumenSurfaceCacheResolution = 0.5f;

	// Legacy initialization.
	{
		// previous behavior was to capture 2d scene captures before cube scene captures.
		CaptureSortPriority = 1;

		// previous behavior was not exposing MotionBlur and Temporal AA in scene capture 2d.
		ShowFlags.TemporalAA = false;
		ShowFlags.MotionBlur = false;

#if WITH_EDITORONLY_DATA
		//Set the Soft Path that will get resolved on instantiation.
		CaptureMeshPath.SetPath(TEXT("/Engine/EditorMeshes/MatineeCam_SM.MatineeCam_SM"));
#endif
	}
}

void USceneCaptureComponent2D::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITORONLY_DATA
	AActor* MyOwner = GetOwner();
	if ((MyOwner != nullptr) && !IsRunningCommandlet())
	{
		if (DrawFrustum == nullptr)
		{
			DrawFrustum = NewObject<UDrawFrustumComponent>(MyOwner, NAME_None, RF_Transactional | RF_TextExportTransient);
			DrawFrustum->SetupAttachment(this);
			DrawFrustum->SetIsVisualizationComponent(true);
			DrawFrustum->CreationMethod = CreationMethod;
			DrawFrustum->RegisterComponentWithWorld(GetWorld());
			UpdateDrawFrustum();
		}
	}
#endif

#if WITH_EDITOR
	// Update content on register to have at least one frames worth of good data.
	// Without updating here this component would not work in a blueprint construction script which recreates the component after each move in the editor
	if (bCaptureOnMovement)
	{
		TileID = 0;
		CaptureSceneDeferred();
	}
#endif
}

void USceneCaptureComponent2D::SendRenderTransform_Concurrent()
{	
	if (bCaptureOnMovement && !bCaptureEveryFrame)
	{
		CaptureSceneDeferred();
	}

	Super::SendRenderTransform_Concurrent();
}

void USceneCaptureComponent2D::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	const int32 NumTiles = GetNumXTiles() * GetNumYTiles();
	check(NumTiles >= 0);

	if (bCaptureEveryFrame || (GetEnableOrthographicTiling() && TileID < NumTiles))
	{
		CaptureSceneDeferred();
	}

	if (!GetEnableOrthographicTiling())
	{
		return;
	}

	if (bCaptureEveryFrame && NumTiles > 0)
	{
		TileID++;
		TileID %= NumTiles;
	} 
	else if (TileID < NumTiles)
	{
		TileID++;
	}
}

void USceneCaptureComponent2D::ResetOrthographicTilingCounter()
{
	TileID = 0;
}

void USceneCaptureComponent2D::SetCameraView(const FMinimalViewInfo& DesiredView)
{
	SetWorldLocation(DesiredView.Location);
	SetWorldRotation(DesiredView.Rotation);

	FOVAngle = DesiredView.FOV;
	ProjectionType = DesiredView.ProjectionMode;
	OrthoWidth = DesiredView.OrthoWidth;

	FirstPersonFieldOfView = DesiredView.FirstPersonFOV;
	FirstPersonScale = DesiredView.FirstPersonScale;
}

void USceneCaptureComponent2D::GetCameraView(float DeltaTime, FMinimalViewInfo& OutMinimalViewInfo)
{
	OutMinimalViewInfo.Location = GetComponentLocation();
	OutMinimalViewInfo.Rotation = GetComponentRotation();
	
	OutMinimalViewInfo.FOV = FOVAngle;
	OutMinimalViewInfo.AspectRatio = TextureTarget ? (float(TextureTarget->SizeX) / TextureTarget->SizeY) : 1.f;
	OutMinimalViewInfo.bConstrainAspectRatio = false;
	OutMinimalViewInfo.ProjectionMode = ProjectionType;
	OutMinimalViewInfo.OrthoWidth = OrthoWidth;
	OutMinimalViewInfo.bAutoCalculateOrthoPlanes = bAutoCalculateOrthoPlanes;
	OutMinimalViewInfo.AutoPlaneShift = AutoPlaneShift;
	OutMinimalViewInfo.bUpdateOrthoPlanes = bUpdateOrthoPlanes;
	OutMinimalViewInfo.bUseCameraHeightAsViewTarget = bUseCameraHeightAsViewTarget;
	OutMinimalViewInfo.FirstPersonFOV = bEnableFirstPersonFieldOfView ? FirstPersonFieldOfView : OutMinimalViewInfo.FOV;
	OutMinimalViewInfo.FirstPersonScale = bEnableFirstPersonScale ? FirstPersonScale : 1.0f;
	OutMinimalViewInfo.bUseFirstPersonParameters = bEnableFirstPersonFieldOfView || bEnableFirstPersonScale;

	OutMinimalViewInfo.ApplyOverscan(Overscan);
	
	if (bAutoCalculateOrthoPlanes)
	{
		if(const AActor* ViewTarget = GetOwner())
		{
			OutMinimalViewInfo.SetCameraToViewTarget(ViewTarget->GetActorLocation());
		}
	}
}

void USceneCaptureComponent2D::CaptureSceneDeferred()
{
	UWorld* World = GetWorld();
	if (World && World->Scene && IsVisible() && !IsCulledByDetailMode())
	{
		// Defer until after updates finish
		// Needs some CS because of parallel updates.
		FScopeLock ScopeLock(&SceneCapturesToUpdateMapCS);
		SceneCapturesToUpdateMap.AddUnique(World, this);
	}	
}

void USceneCaptureComponent2D::CaptureScene()
{
	UWorld* World = GetWorld();
	if (World && World->Scene && IsVisible() && !IsCulledByDetailMode())
	{
		// Explicit CaptureScene blueprint command should always capture or dump (if capture or dump flag is set)
		bSuppressGpuCaptureOrDump = false;

		// We must push any deferred render state recreations before causing any rendering to happen, to make sure that deleted resource references are updated
		World->SendAllEndOfFrameUpdates();

		TUniquePtr<ISceneRenderBuilder> SceneRenderBuilder = ISceneRenderBuilder::Create(World->Scene);
		UpdateSceneCaptureContents(World->Scene, *SceneRenderBuilder);
		SceneRenderBuilder->Execute();
	}

	if (bCaptureEveryFrame)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("CaptureScene", "CaptureScene: Scene capture with bCaptureEveryFrame enabled was told to update - major inefficiency."));
	}
}

void USceneCaptureComponent2D::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
#if WITH_EDITORONLY_DATA
	USceneCaptureComponent2D* This = CastChecked<USceneCaptureComponent2D>(InThis);
	Collector.AddReferencedObject(This->DrawFrustum);
#endif

	Super::AddReferencedObjects(InThis, Collector);
}

void USceneCaptureComponent2D::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

#if WITH_EDITORONLY_DATA
	if (DrawFrustum)
	{
		DrawFrustum->DestroyComponent();
	}
#endif
}

#if WITH_EDITORONLY_DATA
void USceneCaptureComponent2D::UpdateDrawFrustum()
{
	if (DrawFrustum != nullptr)
	{
		const float FrustumDrawDistance = 1000.0f;
		if (ProjectionType == ECameraProjectionMode::Perspective)
		{
			DrawFrustum->FrustumAngle = FOVAngle;
		}
		else
		{
			DrawFrustum->FrustumAngle = -OrthoWidth;
		}

		DrawFrustum->FrustumStartDist = (bOverride_CustomNearClippingPlane) ? CustomNearClippingPlane : GNearClippingPlane;
		// 1000 is the default frustum distance, ideally this would be infinite but that might cause rendering issues
		DrawFrustum->FrustumEndDist = (MaxViewDistanceOverride > DrawFrustum->FrustumStartDist) ? MaxViewDistanceOverride : 1000.0f;
		DrawFrustum->MarkRenderStateDirty();
	}
}
#endif

#if WITH_EDITOR

bool USceneCaptureComponent2D::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (bUseCustomProjectionMatrix 
			&& (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, ProjectionType)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, FOVAngle)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, OrthoWidth)))
		{
			return false;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, FOVAngle))
		{
			return ProjectionType == ECameraProjectionMode::Perspective;
		}
		else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, OrthoWidth))
		{
			return ProjectionType == ECameraProjectionMode::Orthographic;
		}
		else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, CompositeMode))
		{
			return CaptureSource == SCS_SceneColorHDR;
		}

		static IConsoleVariable* ClipPlaneCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AllowGlobalClipPlane"));

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, bEnableClipPlane))
		{
			return ClipPlaneCVar->GetInt() != 0;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, ClipPlaneBase)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, ClipPlaneNormal))
		{
			return bEnableClipPlane && ClipPlaneCVar->GetInt() != 0;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USceneCaptureComponent2D, CustomProjectionMatrix))
		{
			return bUseCustomProjectionMatrix;
		}
	}

	return Super::CanEditChange(InProperty);
}

void USceneCaptureComponent2D::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// AActor::PostEditChange will ForceUpdateComponents()
	Super::PostEditChangeProperty(PropertyChangedEvent);

	TileID = 0;
	CaptureSceneDeferred();

	UpdateDrawFrustum();
}
#endif // WITH_EDITOR

void USceneCaptureComponent2D::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::OrthographicCameraDefaultSettings)
	{
		OrthoWidth = 512.0f;
	}

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::OrthographicAutoNearFarPlane)
	{
		bAutoCalculateOrthoPlanes = false;
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SceneCaptureDefaultSettings
		&& !GetArchetype()->IsInBlueprint())
	{
		// For backwards compatibility, we must preserve the old defaults that do not reflect code overrides in SetupViewFamilyForSceneCapture.
		PostProcessSettings = FPostProcessSettings::GetDefault();
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		PostProcessSettings.OnAfterLoad();
#endif

		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MotionBlurAndTAASupportInSceneCapture2d)
		{
			ShowFlags.TemporalAA = false;
			ShowFlags.MotionBlur = false;
		}

#if WITH_EDITOR
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::OrthographicCameraDefaultSettings && bUseFauxOrthoViewPos)
		{
			FMessageLog("MapCheck").Info()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_UseFauxOrthoViewPosDeprecation", "bUseFauxOrthoViewPos is true but has been deprecated. The setting should be set to false unless any custom usage of this flag is not affected.")))
				->AddToken(FMapErrorToken::Create(FMapErrors::UseFauxOrthoViewPosDeprecation_Warning));
		}
#endif
	}
}

void USceneCaptureComponent2D::UpdateSceneCaptureContents(FSceneInterface* Scene, ISceneRenderBuilder& SceneRenderBuilder)
{
	Scene->UpdateSceneCaptureContents(this, SceneRenderBuilder);
}

bool USceneCaptureComponent2D::GetEnableOrthographicTiling() const
{
	if (!CVarSCOverrideOrthographicTilingValues->GetBool())
	{
		return bEnableOrthographicTiling;
	}

	return CVarSCEnableOrthographicTiling->GetBool();
}

int32 USceneCaptureComponent2D::GetNumXTiles() const
{
	if (!CVarSCOverrideOrthographicTilingValues->GetBool())
	{
		return NumXTiles;
	}
	
	int32 NumXTilesLocal = CVarSCOrthographicNumXTiles->GetInt();
	return FMath::Clamp(NumXTilesLocal, 1, 64);
}

int32 USceneCaptureComponent2D::GetNumYTiles() const
{
	if (!CVarSCOverrideOrthographicTilingValues->GetBool())
	{
		return NumYTiles;
	}

	int32 NumYTilesLocal = CVarSCOrthographicNumYTiles->GetInt();
	return FMath::Clamp(NumYTilesLocal, 1, 64);
}

// -----------------------------------------------

APlanarReflection::APlanarReflection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bShowPreviewPlane_DEPRECATED = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	PlanarReflectionComponent = CreateDefaultSubobject<UPlanarReflectionComponent>(TEXT("NewPlanarReflectionComponent"));
	RootComponent = PlanarReflectionComponent;

	UBoxComponent* DrawInfluenceBox = CreateDefaultSubobject<UBoxComponent>(TEXT("DrawBox0"));
	DrawInfluenceBox->SetupAttachment(PlanarReflectionComponent);
	DrawInfluenceBox->bUseEditorCompositing = true;
	DrawInfluenceBox->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	PlanarReflectionComponent->PreviewBox = DrawInfluenceBox;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (!IsRunningCommandlet() && (SpriteComponent != nullptr))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			FName NAME_ReflectionCapture;
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> DecalTexture;
			FConstructorStatics()
				: NAME_ReflectionCapture(TEXT("ReflectionCapture"))
				, DecalTexture(TEXT("/Engine/EditorResources/S_ReflActorIcon"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.DecalTexture.Get();
		SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->bHiddenInGame = true;
		SpriteComponent->SetUsingAbsoluteScale(true);
		SpriteComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		SpriteComponent->bIsScreenSizeScaled = true;
	}
#endif
}

void APlanarReflection::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FEditorObjectVersion::GUID) < FEditorObjectVersion::ChangeSceneCaptureRootComponent)
	{
		if (PlanarReflectionComponent)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			PlanarReflectionComponent->bShowPreviewPlane = bShowPreviewPlane_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

void APlanarReflection::OnInterpToggle(bool bEnable)
{
	PlanarReflectionComponent->SetVisibility(bEnable);
}

#if WITH_EDITOR
void APlanarReflection::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	Super::EditorApplyScale(FVector(DeltaScale.X, DeltaScale.Y, 0), PivotLocation, bAltDown, bShiftDown, bCtrlDown);

	UPlanarReflectionComponent* ReflectionComponent = GetPlanarReflectionComponent();
	check(ReflectionComponent);
	const FVector ModifiedScale = FVector(0, 0, DeltaScale.Z) * ( AActor::bUsePercentageBasedScaling ? 500.0f : 50.0f );
	FMath::ApplyScaleToFloat(ReflectionComponent->DistanceFromPlaneFadeoutStart, ModifiedScale);
	FMath::ApplyScaleToFloat(ReflectionComponent->DistanceFromPlaneFadeoutEnd, ModifiedScale);
	PostEditChange();
}

#endif

// -----------------------------------------------

// 0 is reserved to mean invalid
int32 NextPlanarReflectionId = 0;

UPlanarReflectionComponent::UPlanarReflectionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bShowPreviewPlane = true;
	bCaptureEveryFrame = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	// Tick in the editor so that bCaptureEveryFrame preview works
	bTickInEditor = true;
	RenderTarget = NULL;
	PrefilterRoughness = .01f;
	PrefilterRoughnessDistance = 10000;
	ScreenPercentage = 50;
	NormalDistortionStrength = 500;
	DistanceFromPlaneFadeStart_DEPRECATED = 400;
	DistanceFromPlaneFadeEnd_DEPRECATED = 600;
	DistanceFromPlaneFadeoutStart = 60;
	DistanceFromPlaneFadeoutEnd = 100;
	AngleFromPlaneFadeStart = 20;
	AngleFromPlaneFadeEnd = 30;
	ProjectionWithExtraFOV[0] = FMatrix::Identity;
	ProjectionWithExtraFOV[1] = FMatrix::Identity;

	// Disable screen space effects that don't work properly with the clip plane
	ShowFlags.SetLightShafts(0);
	ShowFlags.SetContactShadows(0);
	ShowFlags.SetScreenSpaceReflections(0);

	NextPlanarReflectionId++;
	PlanarReflectionId = NextPlanarReflectionId;

#if WITH_EDITORONLY_DATA
	//Set the Soft Path that will get resolved on instantiation.
	CaptureMeshPath.SetPath(TEXT("/Engine/EditorMeshes/PlanarReflectionPlane.PlanarReflectionPlane"));
	CaptureMaterialPath.SetPath(TEXT("/Engine/EditorMeshes/ColorCalibrator/M_ChromeBall.M_ChromeBall"));
#endif
}

void UPlanarReflectionComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITORONLY_DATA
	if (ProxyMeshComponent)
	{
		ProxyMeshComponent->SetMaterial(0, Cast<UMaterial>(CaptureMaterialPath.TryLoad()));
		ProxyMeshComponent->SetVisibleFlag(bShowPreviewPlane);
		ProxyMeshComponent->SetRelativeScale3D(FVector(4, 4, 1));
	}
#endif
}

void UPlanarReflectionComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::ChangedPlanarReflectionFadeDefaults)
	{
		DistanceFromPlaneFadeoutEnd = DistanceFromPlaneFadeEnd_DEPRECATED;
		DistanceFromPlaneFadeoutStart = DistanceFromPlaneFadeStart_DEPRECATED;
	}
}

void UPlanarReflectionComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	UpdatePreviewShape();

	Super::CreateRenderState_Concurrent(Context);

	if (ShouldComponentAddToScene() && ShouldRender())
	{
		SceneProxy = new FPlanarReflectionSceneProxy(this);
		GetWorld()->Scene->AddPlanarReflection(this);
	}
}

void UPlanarReflectionComponent::SendRenderTransform_Concurrent()
{	
	UpdatePreviewShape();

	if (SceneProxy)
	{
		GetWorld()->Scene->UpdatePlanarReflectionTransform(this);
	}

	Super::SendRenderTransform_Concurrent();
}

void UPlanarReflectionComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	if (SceneProxy)
	{
		GetWorld()->Scene->RemovePlanarReflection(this);

		FPlanarReflectionSceneProxy* InSceneProxy = SceneProxy;
		ENQUEUE_RENDER_COMMAND(FDestroyPlanarReflectionCommand)(
			[InSceneProxy](FRHICommandList& RHICmdList)
		{
				delete InSceneProxy;
		});

		SceneProxy = nullptr;
	}
}

bool UPlanarReflectionComponent::ShouldComponentAddToScene() const
{
	bool bSceneAdd = USceneComponent::ShouldComponentAddToScene();

	return bSceneAdd;
}

#if WITH_EDITOR

void UPlanarReflectionComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	for (int32 ViewIndex = 0; ViewIndex < ViewStates.Num(); ViewIndex++)
	{
		const ERHIFeatureLevel::Type FeatureLevel = GetScene() ? GetScene()->GetFeatureLevel() : GMaxRHIFeatureLevel;

		// Recreate the view state to reset temporal history so that property changes can be seen immediately
		ViewStates[ViewIndex].Destroy();
		ViewStates[ViewIndex].Allocate(FeatureLevel);
	}

	if (ProxyMeshComponent)
	{
		ProxyMeshComponent->SetVisibleFlag(bShowPreviewPlane);
		ProxyMeshComponent->MarkRenderStateDirty();
	}
}

#endif

void UPlanarReflectionComponent::BeginDestroy()
{
	if (RenderTarget)
	{
		BeginReleaseResource(RenderTarget);
	}
	
	// Begin a fence to track the progress of the BeginReleaseResource being processed by the RT
	ReleaseResourcesFence.BeginFence();

	Super::BeginDestroy();
}

bool UPlanarReflectionComponent::IsReadyForFinishDestroy()
{
	// Wait until the fence is complete before allowing destruction
	return Super::IsReadyForFinishDestroy() && ReleaseResourcesFence.IsFenceComplete();
}

void UPlanarReflectionComponent::FinishDestroy()
{
	Super::FinishDestroy();

	delete RenderTarget;
	RenderTarget = NULL;
}

void UPlanarReflectionComponent::UpdatePreviewShape()
{
	if (PreviewBox)
	{
		PreviewBox->InitBoxExtent(FVector(500 * 4, 500 * 4, DistanceFromPlaneFadeoutEnd));
	}
}

// -----------------------------------------------


USceneCaptureComponentCube::USceneCaptureComponentCube(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	bTickInEditor = true;
	bCaptureRotation = false;
	ReflectionTimeSliceFirst = 0;
	ReflectionTimeSliceCount = 0;
	ReflectionComponent = nullptr;

	// default to full blend weight..
	PostProcessBlendWeight = 1.0f;

	// Update default post-processing values to reflect default values (previously and still) applied in code.
	PostProcessSettings.DynamicGlobalIlluminationMethod = EDynamicGlobalIlluminationMethod::None;
	PostProcessSettings.ReflectionMethod = EReflectionMethod::None;
	PostProcessSettings.LumenSurfaceCacheResolution = 0.5f;
	PostProcessSettings.VignetteIntensity = 0.0f;
	PostProcessSettings.LumenReflectionsScreenTraces = 0;
	PostProcessSettings.LumenFinalGatherScreenTraces = 0;

#if WITH_EDITORONLY_DATA
	CaptureMeshPath.SetPath(TEXT("/Engine/EditorMeshes/MatineeCam_SM.MatineeCam_SM"));

	DrawFrustum = nullptr;
#endif
}

void USceneCaptureComponentCube::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITORONLY_DATA
	AActor* MyOwner = GetOwner();
	if ((MyOwner != nullptr) && !IsRunningCommandlet())
	{
		if (DrawFrustum == nullptr)
		{
			DrawFrustum = NewObject<UDrawFrustumComponent>(MyOwner, NAME_None, RF_Transactional | RF_TextExportTransient);
			DrawFrustum->SetupAttachment(this);
			DrawFrustum->SetIsVisualizationComponent(true);
			DrawFrustum->CreationMethod = CreationMethod;
			DrawFrustum->RegisterComponentWithWorld(GetWorld());
			UpdateDrawFrustum();
		}
	}
#endif

#if WITH_EDITOR
	// Update content on register to have at least one frames worth of good data.
	// Without updating here this component would not work in a blueprint construction script which recreates the component after each move in the editor
	if (bCaptureOnMovement)
	{
		CaptureSceneDeferred();
	}
#endif
}

void USceneCaptureComponentCube::SendRenderTransform_Concurrent()
{	
	if (bCaptureOnMovement && !bCaptureEveryFrame)
	{
		CaptureSceneDeferred();
	}

	Super::SendRenderTransform_Concurrent();
}

void USceneCaptureComponentCube::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (bCaptureEveryFrame)
	{
		CaptureSceneDeferred();
	}
}

void USceneCaptureComponentCube::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{

#if WITH_EDITORONLY_DATA
	USceneCaptureComponentCube* This = CastChecked<USceneCaptureComponentCube>(InThis);
	Collector.AddReferencedObject(This->DrawFrustum);
#endif

	Super::AddReferencedObjects(InThis, Collector);
}

void USceneCaptureComponentCube::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

#if WITH_EDITORONLY_DATA
	if (DrawFrustum)
	{
		DrawFrustum->DestroyComponent();
	}
#endif
}

#if WITH_EDITORONLY_DATA
void USceneCaptureComponentCube::UpdateDrawFrustum()
{
	if (DrawFrustum != nullptr)
	{
		DrawFrustum->FrustumStartDist = GNearClippingPlane;

		// 1000 is the default frustum distance, ideally this would be infinite but that might cause rendering issues
		const float OldEndDist = DrawFrustum->FrustumEndDist;
		DrawFrustum->FrustumEndDist = (MaxViewDistanceOverride > DrawFrustum->FrustumStartDist)
			? MaxViewDistanceOverride : 1000.0f;

		DrawFrustum->FrustumAngle = 90;

		if (OldEndDist != DrawFrustum->FrustumEndDist)
		{
			DrawFrustum->MarkRenderStateDirty();
		}
	}
}
#endif

void USceneCaptureComponentCube::CaptureSceneDeferred()
{
	UWorld* World = GetWorld();
	if (World && World->Scene && IsVisible() && !IsCulledByDetailMode())
	{
		// Defer until after updates finish
		// Needs some CS because of parallel updates.
		FScopeLock ScopeLock(&SceneCapturesToUpdateMapCS);
		SceneCapturesToUpdateMap.AddUnique(World, this);
	}	
}

void USceneCaptureComponentCube::CaptureScene()
{
	UWorld* World = GetWorld();
	if (World && World->Scene && IsVisible() && !IsCulledByDetailMode())
	{
		// Explicit CaptureScene blueprint command should always capture or dump (if capture or dump flag is set)
		bSuppressGpuCaptureOrDump = false;

		// We must push any deferred render state recreations before causing any rendering to happen, to make sure that deleted resource references are updated
		World->SendAllEndOfFrameUpdates();

		TUniquePtr<ISceneRenderBuilder> SceneRenderBuilder = ISceneRenderBuilder::Create(World->Scene);
		UpdateSceneCaptureContents(World->Scene, *SceneRenderBuilder);
		SceneRenderBuilder->Execute();
	}

	if (bCaptureEveryFrame)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("CaptureScene", "CaptureScene: Scene capture with bCaptureEveryFrame enabled was told to update - major inefficiency."));
	}
}

void USceneCaptureComponentCube::UpdateSceneCaptureContents(FSceneInterface* Scene, ISceneRenderBuilder& SceneRenderBuilder)
{
	Scene->UpdateSceneCaptureContents(this, SceneRenderBuilder);
}

void USceneCaptureComponentCube::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SceneCaptureDefaultSettings
		&& !GetArchetype()->IsInBlueprint())
	{
		// For backwards compatibility, we must preserve the old defaults that do not reflect code overrides in SetupViewFamilyForSceneCapture.
		PostProcessSettings = FPostProcessSettings::GetDefault();
	}

	Super::Serialize(Ar);
}

#if WITH_EDITOR
void USceneCaptureComponentCube::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// AActor::PostEditChange will ForceUpdateComponents()
	Super::PostEditChangeProperty(PropertyChangedEvent);

	CaptureSceneDeferred();

	UpdateDrawFrustum();
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
