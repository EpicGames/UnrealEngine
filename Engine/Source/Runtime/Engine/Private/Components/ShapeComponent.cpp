// Copyright Epic Games, Inc. All Rights Reserved.


#include "Components/ShapeComponent.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "BodySetupEnums.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsInterfaceTypesCore.h"
#include "Serialization/CustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShapeComponent)

// Custom serialization version
struct FShapeComponentCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Reworked AreaClass Implementation
		AreaClassRework,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FShapeComponentCustomVersion() {}
};

const FGuid FShapeComponentCustomVersion::GUID(0xB6E31B1C, 0xD29F11EC, 0x857E9F85, 0x6F9970E2);

// Register the custom version with core
FCustomVersionRegistration GRegisterShapeComponentCustomVersion(FShapeComponentCustomVersion::GUID, FShapeComponentCustomVersion::LatestVersion, TEXT("ShapeComponentVer"));

UShapeComponent::UShapeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static const FName CollisionProfileName("OverlapAllDynamic");
	SetCollisionProfileName(CollisionProfileName);
	BodyInstance.bAutoWeld = true;	//UShapeComponent by default has auto welding

	bHiddenInGame = true;
	bCastDynamicShadow = false;
	bExcludeFromLightAttachmentGroup = true;
	ShapeColor = FColor(223, 149, 157, 255);
	bShouldCollideWhenPlacing = false;

	bUseArchetypeBodySetup = !IsTemplate();
	
	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::Yes;
	bCanEverAffectNavigation = true;
	bDynamicObstacle = false;
	
#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AreaClass = FNavigationSystem::GetDefaultObstacleArea();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

	// Ignore streaming updates since GetUsedMaterials() is not implemented.
	SetIgnoreStreamingManagerUpdate(true);

	bUseSystemDefaultObstacleAreaClass = true;
	AreaClassOverride = nullptr;
}

void UShapeComponent::SetLineThickness(float Thickness)
{
	LineThickness = Thickness;
	MarkRenderStateDirty();
}

FPrimitiveSceneProxy* UShapeComponent::CreateSceneProxy()
{
	check( false && "Subclass needs to Implement this" );
	return NULL;
}

FBoxSphereBounds UShapeComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	check( false && "Subclass needs to Implement this" );
	return FBoxSphereBounds();
}

void UShapeComponent::UpdateBodySetup()
{
	check( false && "Subclass needs to Implement this" );
}

UBodySetup* UShapeComponent::GetBodySetup()
{
	UpdateBodySetup();
	return ShapeBodySetup;
}

void UShapeComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Ar.IsMigratingRemoteObjects() && Ar.IsLoading())
	{
		// ShapeBodySetup exists outside of this hierarchy (it is a runtime instanced SubObject on the CDO)
		// So null it out here so we don't end up with that runtime CDO SubObject from another server.
		// If we migrated, it would show up as a RemoteObject and crash since its Outer is the CDO.
		// Note: We should ban CDO mutation on runtime objects. This is never safe due to delta serialization.
		if (ShapeBodySetup.GetResidence() == EResidence::Remote)
		{
			ShapeBodySetup = nullptr;
		}
	}
#endif

#if WITH_EDITOR
	Ar.UsingCustomVersion(FShapeComponentCustomVersion::GUID);

	if (Ar.IsLoading())
	{
		const int32 Version = Ar.CustomVer(FShapeComponentCustomVersion::GUID);

		// Note this has to be done during Serialize() not on PostLoad(), otherwise a blueprint class (with this component) CDO,
		// can become out of sync with patching AreaClassOverride with AreaClass and causes bugs. This occurs if a level has an instance of the 
		// blueprint class (with this component) and is modified then saved after patching up (but the BP class is not saved and is the 
		// older version still), on the next time the level is loaded.
		if (Version < FShapeComponentCustomVersion::AreaClassRework)
		{
			// If we are loading this object prior to the AreaClass rework then we just use whatever the current AreaClass is as the override.
			bUseSystemDefaultObstacleAreaClass = false;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			AreaClassOverride = AreaClass;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITORONLY_DATA
void UShapeComponent::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UBodySetup::StaticClass()));
}
#endif

#if WITH_EDITOR
void UShapeComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!IsTemplate())
	{
		UpdateBodySetup(); // do this before reregistering components so that new values are used for collision
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

bool UShapeComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	return !bDynamicObstacle;
}

void UShapeComponent::GetNavigationData(FNavigationRelevantData& Data) const
{
	Super::GetNavigationData(Data);

	if (bDynamicObstacle)
	{
		Data.Modifiers.CreateAreaModifiers(this, GetDesiredAreaClass());
	}
}

bool UShapeComponent::IsNavigationRelevant() const
{
	// failed CanEverAffectNavigation() always takes priority
	// dynamic obstacle overrides collision check

	return (bDynamicObstacle && CanEverAffectNavigation()) || Super::IsNavigationRelevant();
}

TSubclassOf<class UNavAreaBase> UShapeComponent::GetDesiredAreaClass() const
{
	return bUseSystemDefaultObstacleAreaClass ?  FNavigationSystem::GetDefaultObstacleArea() : AreaClassOverride;
}

void UShapeComponent::SetAreaClassOverride(TSubclassOf<class UNavAreaBase> InAreaClassOverride)
{
	AreaClassOverride = InAreaClassOverride;
	bUseSystemDefaultObstacleAreaClass = false;
}

void UShapeComponent::SetUseSystemDefaultObstacleAreaClass()
{
	bUseSystemDefaultObstacleAreaClass = true;
}

template <> ENGINE_API void UShapeComponent::AddShapeToGeomArray<FKBoxElem>() { ShapeBodySetup->AggGeom.BoxElems.Add(FKBoxElem()); }
template <> ENGINE_API void UShapeComponent::AddShapeToGeomArray<FKSphereElem>() { ShapeBodySetup->AggGeom.SphereElems.Add(FKSphereElem()); }
template <> ENGINE_API void UShapeComponent::AddShapeToGeomArray<FKSphylElem>() { ShapeBodySetup->AggGeom.SphylElems.Add(FKSphylElem()); }

template <> ENGINE_API
void UShapeComponent::SetShapeToNewGeom<FKBoxElem>(const FPhysicsShapeHandle& Shape)
{
	FPhysicsInterface::SetUserData(Shape, (void*)ShapeBodySetup->AggGeom.BoxElems[0].GetUserData());
}

template <> ENGINE_API
void UShapeComponent::SetShapeToNewGeom<FKSphereElem>(const FPhysicsShapeHandle& Shape)
{
	FPhysicsInterface::SetUserData(Shape, (void*)ShapeBodySetup->AggGeom.SphereElems[0].GetUserData());
}

template <> ENGINE_API
void UShapeComponent::SetShapeToNewGeom<FKSphylElem>(const FPhysicsShapeHandle& Shape)
{
	FPhysicsInterface::SetUserData(Shape, (void*)ShapeBodySetup->AggGeom.SphylElems[0].GetUserData());
}

template <typename ShapeElemType>
void UShapeComponent::CreateShapeBodySetupIfNeeded()
{
	if (!IsValid(ShapeBodySetup))
	{
		ShapeBodySetup = NewObject<UBodySetup>(this, NAME_None, RF_Transient);
		if (GUObjectArray.IsDisregardForGC(this))
		{
			ShapeBodySetup->AddToRoot();
		}

		// If this component is in GC cluster, make sure we add the body setup to it to
		ShapeBodySetup->AddToCluster(this);
		// if we got created outside of game thread, but got added to a cluster, 
		// we no longer need the Async flag
		if (ShapeBodySetup->HasAnyInternalFlags(EInternalObjectFlags::Async) && GUObjectClusters.GetObjectCluster(ShapeBodySetup))
		{
			ShapeBodySetup->ClearInternalFlags(EInternalObjectFlags::Async);
		}
		
		ShapeBodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
		AddShapeToGeomArray<ShapeElemType>();
		ShapeBodySetup->bNeverNeedsCookedCollisionData = true;
		bUseArchetypeBodySetup = false;	//We're making our own body setup, so don't use the archetype's.

		//Update bodyinstance and shapes
		BodyInstance.BodySetup = ShapeBodySetup;
		{
			if(BodyInstance.IsValidBodyInstance())
			{
				FPhysicsCommand::ExecuteWrite(BodyInstance.GetPhysicsActorWeldRoot(), [this](const FPhysicsActorHandle& Actor)
				{
					TArray<FPhysicsShapeHandle> Shapes;
					BodyInstance.GetAllShapes_AssumesLocked(Shapes);

					for(FPhysicsShapeHandle& Shape : Shapes)	//The reason we iterate is we may have multiple scenes and thus multiple shapes, but they are all pointing to the same geometry
					{
						//Update shape with the new body setup. Make sure to only update shapes owned by this body instance
						if(BodyInstance.IsShapeBoundToBody(Shape))
						{
							SetShapeToNewGeom<ShapeElemType>(Shape);
						}
					}
				});
			}
		}
	}
}

//Explicit instantiation of the different shape components
template ENGINE_API void UShapeComponent::CreateShapeBodySetupIfNeeded<FKSphylElem>();
template ENGINE_API void UShapeComponent::CreateShapeBodySetupIfNeeded<FKBoxElem>();
template ENGINE_API void UShapeComponent::CreateShapeBodySetupIfNeeded<FKSphereElem>();
