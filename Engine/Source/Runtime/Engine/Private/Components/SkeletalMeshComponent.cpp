// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnSkeletalComponent.cpp: Actor component implementation.
=============================================================================*/

#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimationSettings.h"
#include "Animation/PoseSnapshot.h"
#include "Containers/Ticker.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "AI/NavigationSystemHelpers.h"
#include "Engine/SkinnedAsset.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Misc/Fork.h"
#include "Particles/ParticleSystemComponent.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimBlueprint.h"
#include "RenderingThread.h"
#include "ShowFlags.h"
#include "SkeletalRender.h"
#include "SkinnedAssetCompiler.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/RenderCommandPipes.h"
#include "AutoRTFM.h"

#include "Logging/MessageLog.h"
#include "Animation/AnimNode_LinkedInputPose.h"

#include "ClothCollisionSource.h"
#include "ClothingSimulationFactory.h"
#include "ClothingSimulationInstance.h"
#include "ClothingSimulationInteractor.h"
#include "ClothingSimulationTeleportHelpers.h"
#include "Features/IModularFeatures.h"
#include "Misc/RuntimeErrors.h"
#include "SkeletalMeshSceneProxy.h"
#include "ContentStreaming.h"
#include "Animation/AnimTrace.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Animation/AnimSubsystem_SharedLinkedAnimLayers.h"
#include "TransformStateStream.h"
#include "UObject/Stack.h"
#if WITH_EDITOR
#include "Engine/PoseWatch.h"
#include "Settings/AnimBlueprintSettings.h"
#include "SkeletalMeshComponentInstanceData.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Misc/ScopeRWLock.h"
#include "Kismet2/BlueprintEditorUtils.h"
#endif

#include "Chaos/Serialization/SerializedDataBuffer.h"
#include "Chaos/Serialization/SerializedMultiPhysicsState.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Physics/Experimental/PhysScene_AsyncPhysicsStateJobQueue.h"
#include "UObject/UObjectMigrationContext.h"

#include "GenericPlatform/GenericPlatformCrashContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshComponent)

LLM_DEFINE_TAG(SkeletalMesh_TransformData);

#define LOCTEXT_NAMESPACE "SkeletalMeshComponent"

TAutoConsoleVariable<int32> CVarUseParallelAnimationEvaluation(TEXT("a.ParallelAnimEvaluation"), 1, TEXT("If 1, animation evaluation will be run across the task graph system. If 0, evaluation will run purely on the game thread"));
TAutoConsoleVariable<int32> CVarUseParallelAnimUpdate(TEXT("a.ParallelAnimUpdate"), 1, TEXT("If != 0, then we update animation blend tree, native update, asset players and montages (is possible) on worker threads."));
TAutoConsoleVariable<int32> CVarForceUseParallelAnimUpdate(TEXT("a.ForceParallelAnimUpdate"), 0, TEXT("If != 0, then we update animations on worker threads regardless of the setting on the project or anim blueprint."));
TAutoConsoleVariable<int32> CVarUseParallelAnimationInterpolation(TEXT("a.ParallelAnimInterpolation"), 1, TEXT("If 1, animation interpolation will be run across the task graph system. If 0, interpolation will run purely on the game thread"));

static TAutoConsoleVariable<float> CVarStallParallelAnimation(
	TEXT("CriticalPathStall.ParallelAnimation"),
	0.0f,
	TEXT("Sleep for the given time in each parallel animation task. Time is given in ms. This is a debug option used for critical path analysis and forcing a change in the critical path."));

static TAutoConsoleVariable<int32> CVarHiPriSkinnedMeshesTicks(
	TEXT("tick.HiPriSkinnedMeshes"),
	1,
	TEXT("If > 0, then schedule the skinned component ticks in a tick group before other ticks."));

static bool bCVarUninitializeInstanceOnNewSingleNodeInstance = true;
static FAutoConsoleVariableRef CVarUninitializeSingleNodeInstanceOnNewInstance(
	TEXT("a.UninitializeSingleNodeInstanceOnNewInstance"),
	bCVarUninitializeInstanceOnNewSingleNodeInstance,
	TEXT("Whether the current anim instance (if any) should be uninitialized before being assigned a new instance."));

// Deprecated. Please switch to ANIM_SKINNED_ASSET_ISPC_ENABLED_DEFAULT and "a.SkinnedAsset.ISPC", see SkinnedAsset.cpp.
#if !defined(ANIM_SKELETAL_MESH_ISPC_ENABLED_DEFAULT)
#define ANIM_SKELETAL_MESH_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
bool bAnim_SkeletalMesh_ISPC_Enabled = INTEL_ISPC && ANIM_SKELETAL_MESH_ISPC_ENABLED_DEFAULT;
#else
bool bAnim_SkeletalMesh_ISPC_Enabled = ANIM_SKELETAL_MESH_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarAnimSkeletalMeshISPCEnabled(TEXT("a.SkeletalMesh.ISPC"), bAnim_SkeletalMesh_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in animation skeletal mesh components. Deprecated, please switch to a.SkinnedAsset.ISPC"));
#endif

static TAutoConsoleVariable<int32> CVarCacheLocalSpaceBounds(
	TEXT("a.CacheLocalSpaceBounds"),
	1,
	TEXT("If 1 (default) local-space bounds are calculated and cached, otherwise worldspace bounds are built and cached (and inverse transformed to produce local bounds)."));

#if WITH_EDITOR
static bool GExtendSkeletalMeshDataCleanupRange = false;
static FAutoConsoleVariableRef CVarExtendSkeletalMeshDataCleanupRange(
	TEXT("a.ExtendSkeletalMeshDataCleanupRange"),
	GExtendSkeletalMeshDataCleanupRange,
	TEXT("Extend the version check for skel mesh components with possible broken skeletal mesh asset properties to include 5.6.0"));
#endif

DECLARE_CYCLE_STAT_EXTERN(TEXT("Anim Instance Spawn Time"), STAT_AnimSpawnTime, STATGROUP_Anim, );
DEFINE_STAT(STAT_AnimSpawnTime);
DEFINE_STAT(STAT_PostAnimEvaluation);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

static bool GParallelAnimCompletionTaskHighPriority = true;
static FAutoConsoleVariableRef CVarParallelAnimCompletionTaskHighPriority(
	TEXT("TaskGraph.TaskPriorities.ParallelAnimCompletionTaskHighPriority"),
	GParallelAnimCompletionTaskHighPriority,
	TEXT("Allows parallel anim completion tasks to take priority on the GT so further work (if needed) can be kicked off earlier."),
	ECVF_Default
);

FAutoConsoleTaskPriority CPrio_ParallelAnimationEvaluationTask(
	TEXT("TaskGraph.TaskPriorities.ParallelAnimationEvaluationTask"),
	TEXT("Task and thread priority for FParallelAnimationEvaluationTask"),
	ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
	);

/** Static Multicaster fired when SkeletalMeshComponent finalizes the regeneration of the required bones list for the current LOD*/
/*static*/ FOnLODRequiredBonesUpdateMulticast USkeletalMeshComponent::OnLODRequiredBonesUpdate_Static;

#if WITH_EDITOR
namespace UE::Private
{
	extern UNREALED_API bool GraphContainsSetSkeletalMeshAssetNode(UEdGraph* Graph);
}

struct SkeletalMeshUCSModifiedDataCleanupHelper
{
	static void CleanupData(USkeletalMeshComponent* Component)
	{
		static const FName SkeletalMeshPropertyName("SkeletalMesh");

		AActor* OwningActor = Component->GetOwner();
		if (!OwningActor || !OwningActor->GetLevel())
		{
			return;
		}

		UBlueprint* Blueprint = Cast<UBlueprint>(OwningActor->GetClass()->ClassGeneratedBy);
		if (!Blueprint)
		{
			return;
		}

		UEdGraph* CSGraph = FBlueprintEditorUtils::FindUserConstructionScript(Blueprint);
		if (!CSGraph || UE::Private::GraphContainsSetSkeletalMeshAssetNode(CSGraph))
		{
			return;
		}
		
		// Check if the property is actually overridden, if not, we can early out
		UClass* ComponentClass = Component->GetClass();
		FProperty* MeshAssetProperty = ComponentClass->FindPropertyByName(SkeletalMeshPropertyName);
		if (MeshAssetProperty)
		{	
			UObject* ComponentArchetype = Component->GetArchetype();
			uint8* DataPtr = MeshAssetProperty->ContainerPtrToValuePtr<uint8>((uint8*)Component);
			uint8* DefaultValue = MeshAssetProperty->ContainerPtrToValuePtrForDefaults<uint8>(ComponentClass, (uint8*)ComponentArchetype);

			if (MeshAssetProperty->Identical(DataPtr, DefaultValue, PPF_DeepCompareInstances))
			{
				return;
			}
		}

		bool bFoundUCSEntry = false;
		TArray<FSimpleMemberReference> UCSModifiedProperties;

		{
			UE::TRWScopeLock Lock(Component->AllUCSModifiedPropertiesLock, SLT_Write);
			TArray<FSimpleMemberReference>* UCSModifiedPropertiesPtr = Component->AllUCSModifiedProperties.Find(Component);
			if (UCSModifiedPropertiesPtr)
			{
				for (int32 Index = 0; Index < UCSModifiedPropertiesPtr->Num(); Index++)
				{
					FSimpleMemberReference& Reference = (*UCSModifiedPropertiesPtr)[Index];
					if (Reference.MemberName == SkeletalMeshPropertyName)
					{
						bFoundUCSEntry = true;
						UCSModifiedProperties = *UCSModifiedPropertiesPtr;
						UCSModifiedProperties.RemoveAt(Index);
						break;
					}
				}
			}

			if (bFoundUCSEntry)
			{
				Component->AllUCSModifiedProperties.Remove(Component);
				if (UCSModifiedProperties.Num() > 0)
				{
					Component->AllUCSModifiedProperties.Add(Component, MoveTemp(UCSModifiedProperties));
				}
			}
		}
	}
};
#endif

#if !UE_BUILD_SHIPPING
CSV_DEFINE_CATEGORY(AnimationParallelEvaluation, true);
struct FParallelAnimationEvaluationStats
{
	static FParallelAnimationEvaluationStats& Get()
	{
		static FParallelAnimationEvaluationStats Stats;
		return Stats;
	}

	void AddTiming(float Value)
	{
		AtomicFloatAddRelaxed(TotalTime, Value);
		AtomicFloatMinRelaxed(MinTime, Value);
		AtomicFloatMaxRelaxed(MaxTime, Value);
		NumberOfTasks.fetch_add(1, std::memory_order_release);
	}

private:
	FParallelAnimationEvaluationStats()
	{
		EndOfFrameHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FParallelAnimationEvaluationStats::OnEndFrame);
		MinTime.store(TNumericLimits<float>::Max(), std::memory_order_relaxed);
		MaxTime.store(0.f, std::memory_order_relaxed);
		TotalTime.store(0.f, std::memory_order_relaxed);
		NumberOfTasks.store(0, std::memory_order_release);
	}
	
	~FParallelAnimationEvaluationStats()
	{
		FCoreDelegates::OnEndFrame.Remove(EndOfFrameHandle);
	}

	void OnEndFrame()
	{
		// Only set stats when a task has actually run
		const int32 NumTasks = NumberOfTasks.load(std::memory_order_acquire);
		if (NumTasks > 0)
		{	
			const float Min = MinTime.load(std::memory_order_relaxed);
			const float Max = MaxTime.load(std::memory_order_relaxed);
			const float Total = TotalTime.load(std::memory_order_relaxed);
			const float AvgTime = FMath::IsNearlyZero(Total) ? 0.f : Total / NumTasks;

		    CSV_CUSTOM_STAT(AnimationParallelEvaluation, TotalTaskTime, Total, ECsvCustomStatOp::Set);
		    CSV_CUSTOM_STAT(AnimationParallelEvaluation, AverageTaskTime, AvgTime, ECsvCustomStatOp::Set);
		    CSV_CUSTOM_STAT(AnimationParallelEvaluation, NumberOfTasks, NumTasks, ECsvCustomStatOp::Set);
				    
		    CSV_CUSTOM_STAT(AnimationParallelEvaluation, MinTaskTime, Min, ECsvCustomStatOp::Set);
		    CSV_CUSTOM_STAT(AnimationParallelEvaluation, MaxTaskTime, Max, ECsvCustomStatOp::Set);
		}

		MinTime.store(TNumericLimits<float>::Max(), std::memory_order_relaxed);
		MaxTime.store(0.f, std::memory_order_relaxed);
		TotalTime.store(0.f, std::memory_order_relaxed);
		NumberOfTasks.store(0, std::memory_order_release);
	}

	static void AtomicFloatAddRelaxed(std::atomic<float>& Atomic, float Value)
	{
		float OldAtomic = Atomic.load(std::memory_order_relaxed);
		float NewAtomic;
		do
		{
			NewAtomic = OldAtomic + Value;
		} while (!Atomic.compare_exchange_weak(OldAtomic, NewAtomic, std::memory_order_relaxed));
	}

	static void AtomicFloatMinRelaxed(std::atomic<float>& Atomic, float Value)
	{
		float OldAtomic = Atomic.load(std::memory_order_relaxed);
		float NewAtomic;
		do
		{
			NewAtomic = (OldAtomic < Value) ? OldAtomic : Value;
		} while (!Atomic.compare_exchange_weak(OldAtomic, NewAtomic, std::memory_order_relaxed));
	}

	static void AtomicFloatMaxRelaxed(std::atomic<float>& Atomic, float Value)
	{
		float OldAtomic = Atomic.load(std::memory_order_relaxed);
		float NewAtomic;
		do
		{
			NewAtomic = (OldAtomic < Value) ? Value : OldAtomic;
		} while (!Atomic.compare_exchange_weak(OldAtomic, NewAtomic, std::memory_order_relaxed));
	}

	std::atomic<float> MinTime;
	std::atomic<float> MaxTime;
	std::atomic<float> TotalTime;
	std::atomic<int32> NumberOfTasks;

	FDelegateHandle EndOfFrameHandle;
};

// When the object system has been completely loaded, register the OnEndFrame delegate
static FDelayedAutoRegisterHelper GParallelAnimationEvaluationStatsHelper(EDelayedRegisterRunPhase::EndOfEngineInit, []() -> void
{
	FParallelAnimationEvaluationStats::Get();
});

#endif

class FParallelAnimationEvaluationTask
{
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

public:
	FParallelAnimationEvaluationTask(TWeakObjectPtr<USkeletalMeshComponent> InSkeletalMeshComponent)
		: SkeletalMeshComponent(InSkeletalMeshComponent)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParallelAnimationEvaluationTask, STATGROUP_TaskGraphTasks);
	}
	static FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_ParallelAnimationEvaluationTask.Get();
	}
	static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (USkeletalMeshComponent* Comp = SkeletalMeshComponent.Get())
		{
			UE_LOG_CONTEXT("SkeletalMeshComponent", FSoftObjectPath(Comp));
			FScopedFTZFloatMode FTZ;

			FScopeCycleCounterUObject ContextScope(Comp);
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
			float Stall = CVarStallParallelAnimation.GetValueOnAnyThread();
			if (Stall > 0.0f)
			{
				FPlatformProcess::Sleep(Stall / 1000.0f);
			}
#endif
			if (CurrentThread != ENamedThreads::GameThread)
			{
				GInitRunaway();
			}

#if !UE_BUILD_SHIPPING			
			const uint64 StartTime = FPlatformTime::Cycles64();
			Comp->ParallelAnimationEvaluation();
			const float EvaluationTimeMS = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartTime);
			FParallelAnimationEvaluationStats::Get().AddTiming(EvaluationTimeMS);
#else
			Comp->ParallelAnimationEvaluation();
#endif
		}
	}
};

class FParallelAnimationCompletionTask
{
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

public:
	FParallelAnimationCompletionTask(TWeakObjectPtr<USkeletalMeshComponent> InSkeletalMeshComponent)
		: SkeletalMeshComponent(InSkeletalMeshComponent)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParallelAnimationCompletionTask, STATGROUP_TaskGraphTasks);
	}
	static ENamedThreads::Type GetDesiredThread()
	{
		if (GParallelAnimCompletionTaskHighPriority)
		{
			return static_cast<ENamedThreads::Type>(ENamedThreads::GameThread | ENamedThreads::HighTaskPriority);
		}
		return ENamedThreads::GameThread;
	}
	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimGameThreadTime);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Animation);

		if (USkeletalMeshComponent* Comp = SkeletalMeshComponent.Get())
		{
			FScopedFTZFloatMode FTZ;

			FScopeCycleCounterUObject ComponentScope(Comp);
			FScopeCycleCounterUObject MeshScope(Comp->GetSkeletalMeshAsset());

			if(IsValidRef(Comp->ParallelAnimationEvaluationTask))
			{
				const bool bPerformPostAnimEvaluation = true;
				Comp->CompleteParallelAnimationEvaluation(bPerformPostAnimEvaluation);
			}
		}
	}
};

USkeletalMeshComponent::USkeletalMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bTickEvenWhenPaused = false;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;

	bWantsInitializeComponent = true;
	GlobalAnimRateScale = 1.0f;
	bNoSkeletonUpdate = false;
	VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	KinematicBonesUpdateType = EKinematicBonesUpdateToPhysics::SkipSimulatingBones;
	PhysicsTransformUpdateMode = EPhysicsTransformUpdateMode::SimulationUpatesComponentTransform;
	SetGenerateOverlapEvents(false);
	LineCheckBoundsScale = FVector(1.0f, 1.0f, 1.0f);

	EndPhysicsTickFunction.TickGroup = TG_EndPhysics;
	EndPhysicsTickFunction.bCanEverTick = true;
	EndPhysicsTickFunction.bStartWithTickEnabled = true;
	EndPhysicsTickFunction.bRunTransactionally = true;

	ClothTickFunction.TickGroup = TG_PrePhysics;
	ClothTickFunction.EndTickGroup = TG_PostPhysics;
	ClothTickFunction.bCanEverTick = true;

	// End tick runs after every other tick group to wait on the parallel cloth task without pumping the EOF queue.
	ClothEndTickFunction.TickGroup = TG_LastDemotable;
	ClothEndTickFunction.EndTickGroup = TG_LastDemotable;
	ClothEndTickFunction.bCanEverTick = true;
	ClothEndTickFunction.bStartWithTickEnabled = true;

	bWaitForParallelClothTask = false;
	bNotifySyncComponentToRBPhysics = false;

	ClothMaxDistanceScale = 1.0f;
	bResetAfterTeleport = true;
	TeleportDistanceThreshold = 300.0f;
	TeleportRotationThreshold = 0.0f;	// angles in degree, disabled by default
	ClothBlendWeight = 1.0f;

	ClothTeleportMode = EClothingTeleportMode::None;
	PrevRootBoneMatrix = GetBoneMatrix(0); // save the root bone transform
	PrevRootRigidBodyScale = FVector3f(GetRelativeScale3D());

	// pre-compute cloth teleport thresholds for performance
	ClothTeleportDistThresholdSquared = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportDistanceThresholdSquared(TeleportDistanceThreshold);
	ClothTeleportCosineThresholdInRad = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportCosineRotationThreshold(TeleportRotationThreshold);

	bBindClothToLeaderComponent = false;
	bClothingSimulationSuspended = false;

#if WITH_EDITORONLY_DATA
	DefaultPlayRate_DEPRECATED = 1.0f;
	bDefaultPlaying_DEPRECATED = true;
	bOverrideDefaultAnimatingRig = false;
#endif
	bEnablePhysicsOnDedicatedServer = UPhysicsSettings::Get()->bSimulateSkeletalMeshOnDedicatedServer;
	bEnableUpdateRateOptimizations = false;
	RagdollAggregateThreshold = UPhysicsSettings::Get()->RagdollAggregateThreshold;

	bSuppressNotifyEventDispatch = false;

	bUpdateMeshWhenKinematic = false;
	bEnableAnimation = true;

	LastPoseTickFrame = 0u;

	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::Yes;

	bTickInEditor = true;

	CachedAnimCurveUidVersion = 0;
	ResetRootBodyIndex();

	bAllowClothActors = true;
	bPostEvaluatingAnimation = false;
	bValidMeshPoseTransformsUpToDate = false;
	bDisableAnimDynamicsAnimNode = false;
	bAllowAnimCurveEvaluation = true;
	bDisablePostProcessBlueprint = false;

	// By default enable overlaps when blending physics - user can disable if they are sure it's unnecessary
	bUpdateOverlapsOnAnimationFinalize = true;

	bPropagateCurvesToFollowers = false;

	bSkipKinematicUpdateWhenInterpolating = false;
	bSkipBoundsUpdateWhenInterpolating = false;

	DeferredKinematicUpdateIndex = INDEX_NONE;
}

USkeletalMesh* USkeletalMeshComponent::GetSkeletalMeshAsset() const
{
	return Cast<USkeletalMesh>(GetSkinnedAsset());
}

void USkeletalMeshComponent::RefreshSkeletalMeshAsset()
{
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NotifyIfSkinnedAssetChanged();
	SkeletalMeshAsset = GetSkeletalMeshAsset();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

TSharedPtr<FBoneContainer> USkeletalMeshComponent::GetSharedRequiredBones()
{
	if (!SharedRequiredBones)
	{
		SharedRequiredBones = MakeShared<FBoneContainer>();
	}

	return SharedRequiredBones;
}

void USkeletalMeshComponent::Serialize(FArchive& Ar)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
		if (Ar.IsSaving())
		{
			if ((NULL != AnimationBlueprint_DEPRECATED) && (NULL == AnimBlueprintGeneratedClass))
			{
				AnimBlueprintGeneratedClass = Cast<UAnimBlueprintGeneratedClass>(AnimationBlueprint_DEPRECATED->GeneratedClass);
			}
		}
#endif

	Super::Serialize(Ar);

#if WITH_EDITOR	
	// Fix for UE-241328. 5.5.0 and 5.5.1 introduced a bug where skeletal mesh asset properties on blueprint added skeletal mesh components would
	// be incorrectly added to the global UCS modified property list. This affected serialization of overrides of those properties. The fix attempts
	// to clean up the UCS modified property list, removing the invalid properties and leaving those that are valid (ie. are set via UCS).
	// It's possible that some packages saved in 5.6.0 could be affected by the bug. This would happen when building from source if an integration
	// from Main had been made after the original change was introduced in Release-5.5. In these cases, the GExtendSkeletalMeshDataCleanupRange 
	// cvar can be enabled so the fix up is inclusive of 5.6.0 packages.
	if (Ar.IsLoading())
	{
		if (CreationMethod == EComponentCreationMethod::SimpleConstructionScript || CreationMethod == EComponentCreationMethod::UserConstructionScript)
		{
			FEngineVersionBase ArVersion = Ar.EngineVer();
			uint16 Major = ArVersion.GetMajor();
			uint16 Minor = ArVersion.GetMinor();
			uint16 Patch = ArVersion.GetPatch();
			if (Major == 5 && ((GExtendSkeletalMeshDataCleanupRange && Minor == 6 && Patch == 0) || (Minor == 5 && (Patch == 0 || Patch == 1))))
			{
				SkeletalMeshUCSModifiedDataCleanupHelper::CleanupData(this);
			}
		}
	}	
#endif

	// to count memory : TODO: REMOVE?
	if (Ar.IsCountingMemory())
	{
		BoneSpaceTransforms.CountBytes(Ar);
		RequiredBones.CountBytes(Ar);
	}

	if (Ar.UEVer() < VER_UE4_REMOVE_SKELETALMESH_COMPONENT_BODYSETUP_SERIALIZATION)
	{
		//we used to serialize bodysetup of skeletal mesh component. We no longer do this, but need to not break existing content
		if (bEnablePerPolyCollision)
		{
			Ar << BodySetup;
		}
	}

	// Since we separated simulation vs blending
	// if simulation is on when loaded, just set blendphysics to be true
	if (BodyInstance.bSimulatePhysics)
	{
		bBlendPhysics = true;
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && (Ar.UEVer() < VER_UE4_EDITORONLY_BLUEPRINTS))
	{
		if ((NULL != AnimationBlueprint_DEPRECATED))
		{
			// Migrate the class from the animation blueprint once, and null the value so we never get in again
			AnimBlueprintGeneratedClass = Cast<UAnimBlueprintGeneratedClass>(AnimationBlueprint_DEPRECATED->GeneratedClass);
			AnimationBlueprint_DEPRECATED = NULL;
		}
	}

	if (Ar.IsLoading() && (Ar.UEVer() < VER_UE4_NO_ANIM_BP_CLASS_IN_GAMEPLAY_CODE))
	{
		if (nullptr != AnimBlueprintGeneratedClass)
		{
			AnimClass = AnimBlueprintGeneratedClass;
		}
	}

	if (Ar.IsLoading() && AnimBlueprintGeneratedClass)
	{
		AnimBlueprintGeneratedClass = nullptr;
	}
#endif

	if (Ar.IsLoading() && (Ar.UEVer() < VER_UE4_AUTO_WELDING))
	{
		BodyInstance.bAutoWeld = false;
	}

	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::RenameDisableAnimCurvesToAllowAnimCurveEvaluation)
	{
		bAllowAnimCurveEvaluation = !bDisableAnimCurves_DEPRECATED;
	}
#endif

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void USkeletalMeshComponent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	RefreshSkeletalMeshAsset();
}

void USkeletalMeshComponent::PostLoad()
{
	Super::PostLoad();

	// We know for sure that an override was set if this is non-zero.
	if(MinLodModel > 0)
	{
		bOverrideMinLod = true;
	}

	// Update property alias
	RefreshSkeletalMeshAsset();
}

void USkeletalMeshComponent::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITORONLY_DATA
		if (!GetDefault<UAnimBlueprintSettings>()->bAllowAnimBlueprints && AnimationMode == EAnimationMode::AnimationBlueprint)
		{
			AnimationMode = EAnimationMode::AnimationSingleNode;
		}
#endif
	}
}

#if WITH_EDITOR
void USkeletalMeshComponent::PostEditImport()
{
	Super::PostEditImport();

	// Update property alias after a duplicate (PostDuplicate not called on actor components)
	RefreshSkeletalMeshAsset();
}
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
void USkeletalMeshComponent::PostMigrate(const struct FUObjectMigrationContext& MigrationContext)
{
	// When OnRegister gets called from PostMigrate, tell it to not touch the AnimInstance by setting our mode to Custom
	EAnimationMode::Type RestoreAnimationMode = AnimationMode;
	AnimationMode = EAnimationMode::AnimationCustomMode;

	Super::PostMigrate(MigrationContext);

	AnimationMode = RestoreAnimationMode;

	if (!HasValidPhysicsState())
	{
		// If PerPolyCollision is disabled, the primitive component physics state initialization is skipped, therefore we need to listen to
		// a skeletal mesh component specific delegate
		if (!bEnablePerPolyCollision)
		{
			OnComponentPhysicsStateChanged.RemoveDynamic(this, &USkeletalMeshComponent::HandlePostMigrationPhysicsStateChanged);

			RegisterOnPhysicsCreatedDelegate(FOnSkelMeshPhysicsCreated::CreateLambda([WeakThis = TWeakObjectPtr<USkeletalMeshComponent>(this)]()
			{
				if (USkeletalMeshComponent* SKComponent = WeakThis.Get())
				{
					SKComponent->HandlePostMigrationPhysicsStateChanged(SKComponent, EComponentPhysicsStateChange::Created);
				}
			}));
		}
	}
}
#endif

void USkeletalMeshComponent::RegisterComponentTickFunctions(bool bRegister)
{
	Super::RegisterComponentTickFunctions(bRegister);
	const bool bDoHiPri = CVarHiPriSkinnedMeshesTicks.GetValueOnGameThread() > 0;
	if (PrimaryComponentTick.bHighPriority != bDoHiPri)
	{
		// Note that if animation is so long that we are blocked in EndPhysics we may want to reduce the priority. However, there is a risk that this function will not go wide early enough.
		// This requires profiling and is very game dependent so cvar for now makes sense
		PrimaryComponentTick.SetPriorityIncludingPrerequisites(bDoHiPri);
	}

	UpdateEndPhysicsTickRegisteredState();
	UpdateClothTickRegisteredState();
}

void USkeletalMeshComponent::SetComponentTickEnabled(bool bEnabled)
{
	Super::SetComponentTickEnabled(bEnabled);

	// Unregister the cloth tick functions when the component's tick is being disabled (they will be re-enabled at the next component tick update once it runs again)
	if (!bEnabled)
	{
		RegisterClothTick(false);
		RegisterClothEndTick(false);
	}
}

void USkeletalMeshComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	ClearAnimScriptInstance();

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

#if WITH_EDITOR
TStructOnScope<FActorComponentInstanceData> USkeletalMeshComponent::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData, FSkeletalMeshComponentInstanceData>(this);
}
#endif // WITH_EDITOR

void USkeletalMeshComponent::RegisterEndPhysicsTick(bool bRegister)
{
	if (bRegister != EndPhysicsTickFunction.IsTickFunctionRegistered())
	{
		if (bRegister)
		{
			UWorld* World = GetWorld();
			if (World->EndPhysicsTickFunction.IsTickFunctionRegistered() && SetupActorComponentTickFunction(&EndPhysicsTickFunction))
			{
				EndPhysicsTickFunction.Target = this;
				// Make sure our EndPhysicsTick gets called after physics simulation is finished
				if (World != nullptr)
				{
					EndPhysicsTickFunction.AddPrerequisite(World, World->EndPhysicsTickFunction);
				}
			}
		}
		else
		{
			EndPhysicsTickFunction.UnRegisterTickFunction();
		}
	}
}

void USkeletalMeshComponent::RegisterClothTick(bool bRegister)
{
	if (bRegister != ClothTickFunction.IsTickFunctionRegistered())
	{
		if (bRegister)
		{
			if (SetupActorComponentTickFunction(&ClothTickFunction))
			{
				ClothTickFunction.Target = this;
				ClothTickFunction.AddPrerequisite(this, PrimaryComponentTick);
				ClothTickFunction.AddPrerequisite(this, EndPhysicsTickFunction);	//If this tick function is running it means that we are doing physics blending so we should wait for its results
			}
		}
		else
		{
			ClothTickFunction.UnRegisterTickFunction();
		}
	}
}

void USkeletalMeshComponent::RegisterClothEndTick(bool bRegister)
{
	if (bRegister != ClothEndTickFunction.IsTickFunctionRegistered())
	{
		if (bRegister)
		{
			if (SetupActorComponentTickFunction(&ClothEndTickFunction))
			{
				ClothEndTickFunction.Target = this;
				ClothEndTickFunction.AddPrerequisite(this, ClothTickFunction);
			}
		}
		else
		{
			ClothEndTickFunction.UnRegisterTickFunction();
		}
	}
}

bool USkeletalMeshComponent::ShouldRunEndPhysicsTick() const
{
	return	(bEnablePhysicsOnDedicatedServer || !IsNetMode(NM_DedicatedServer)) && // Early out if we are on a dedicated server and not running physics.
			((IsSimulatingPhysics() && RigidBodyIsAwake()) || ShouldBlendPhysicsBones());
}

void USkeletalMeshComponent::UpdateEndPhysicsTickRegisteredState()
{
	RegisterEndPhysicsTick(PrimaryComponentTick.IsTickFunctionRegistered() && ShouldRunEndPhysicsTick());
}

bool USkeletalMeshComponent::ShouldRunClothTick() const
{
	if(bClothingSimulationSuspended)
	{
		return false;
	}

	if(!CanSimulateClothing())
	{
		return false;
	}

	if (GetSkeletalMeshAsset()->HasActiveClothingAssetsForLOD(GetPredictedLODLevel()))
	{
		return true;
	}

	return	false;
}

extern TAutoConsoleVariable<int32> CVarEnableClothPhysics;
extern TAutoConsoleVariable<int32> CVarClothPhysicsLODThreshold;

bool USkeletalMeshComponent::CanSimulateClothing() const
{
	if (!GetSkeletalMeshAsset() || !bAllowClothActors || !CVarEnableClothPhysics.GetValueOnAnyThread())
	{
		return false;
	}

	const int32 LODThreshold = CVarClothPhysicsLODThreshold.GetValueOnAnyThread();
	if (LODThreshold >= 0)
	{
		const int32 LODIndex = GetPredictedLODLevel();
		if (LODIndex < 0 || LODIndex > LODThreshold)
		{
			return false;
		}
	}

	return GetSkeletalMeshAsset()->HasActiveClothingAssets() && !IsNetMode(NM_DedicatedServer);
}

void USkeletalMeshComponent::UpdateClothTickRegisteredState()
{
	const bool bShouldRunClothTick = PrimaryComponentTick.IsTickFunctionRegistered() && ShouldRunClothTick();
	RegisterClothTick(bShouldRunClothTick);
	// The end tick owns the wait on the parallel cloth task when the cloth tick function isn't doing it itself.
	// Skip it when the in-tick wait is on (nothing to do here), or when the end-tick path has been disabled via CVar (the wait falls back to OnPreEndOfFrameSync).
	RegisterClothEndTick(bShouldRunClothTick && !ShouldWaitForClothInTickFunction() && ShouldUseClothEndTickWait());
}

void USkeletalMeshComponent::FinalizePoseEvaluationResult(const USkeletalMesh* InMesh, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FCompactPose& InFinalPose) const
{
	const TArray<FTransform>& RefBonePose = InMesh->GetRefSkeleton().GetRefBonePose();
	if(InFinalPose.IsValid() && InFinalPose.GetNumBones() > 0)
	{
		InFinalPose.NormalizeRotations();

		// Bone index array is in increasing order, so we can fill gaps with reference pose
		const auto FillReferencePose = [&](int32 BeginIndex, int32 EndIndex)
		{
			for (int32 MeshPoseIndex = BeginIndex; MeshPoseIndex < EndIndex; ++MeshPoseIndex)
			{
				OutBoneSpaceTransforms[MeshPoseIndex] = RefBonePose[MeshPoseIndex];
			}
		};

		int32 LastMeshPoseIndex = 0;
		const int32 BoneCount = RefBonePose.Num();
		OutBoneSpaceTransforms.SetNum(BoneCount);
		for (const FCompactPoseBoneIndex BoneIndex : InFinalPose.ForEachBoneIndex())
		{
			const int32 MeshPoseIndex = InFinalPose.GetBoneContainer().MakeMeshPoseIndex(BoneIndex).GetInt();
			FillReferencePose(LastMeshPoseIndex, MeshPoseIndex);
			OutBoneSpaceTransforms[MeshPoseIndex] = InFinalPose[BoneIndex];
			LastMeshPoseIndex = MeshPoseIndex + 1;
		}
		FillReferencePose(LastMeshPoseIndex, BoneCount);
	}
	else
	{
		OutBoneSpaceTransforms = RefBonePose;
	}
	OutRootBoneTranslation = OutBoneSpaceTransforms[0].GetTranslation() - RefBonePose[0].GetTranslation();
}

void USkeletalMeshComponent::FinalizeAttributeEvaluationResults(const FBoneContainer& BoneContainer, const UE::Anim::FHeapAttributeContainer& FinalContainer,
	UE::Anim::FMeshAttributeContainer& OutContainer) const
{
	OutContainer.CopyFrom(FinalContainer, BoneContainer);
}

bool USkeletalMeshComponent::NeedToSpawnAnimScriptInstance() const
{
	if (!IsRegistered() && !IsPreRegistering())
	{
		return false;
	}

	USkeletalMesh* MeshAsset = GetSkeletalMeshAsset();
	IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(AnimClass);
	USkeleton* AnimSkeleton = (AnimClassInterface) ? AnimClassInterface->GetTargetSkeleton() : nullptr;
	if(AnimSkeleton == nullptr)
	{
		// Fall back to mesh skeleton (anim BP could have been a template)
		AnimSkeleton = MeshAsset ? MeshAsset->GetSkeleton() : nullptr;
	}
	const bool bSkeletonsExist = (MeshAsset && AnimSkeleton && MeshAsset->GetSkeleton());
	const bool bSkelMeshCompatible = (MeshAsset && AnimSkeleton) ? AnimSkeleton->IsCompatibleMesh(MeshAsset, false) : false;
	const bool bAnimSkelValid = !AnimClassInterface || (bSkeletonsExist && bSkelMeshCompatible);

	if (AnimationMode == EAnimationMode::AnimationBlueprint && AnimClass && bAnimSkelValid)
	{
		// Check for an 'invalid' AnimScriptInstance:
		// - Could be NULL (in the case of 'standard' first-time initialization)
		// - Could have a different class (in the case where the active anim BP has changed)
		// - Could have a different outer (in the case where an actor has been spawned using an existing actor as a template, as the component is shallow copied directly from the template)
		if ( (AnimScriptInstance == nullptr) || (AnimScriptInstance->GetClass() != AnimClass) || AnimScriptInstance->GetOuter() != this )
		{
			return true;
		}
	}

	return false;
}

bool USkeletalMeshComponent::NeedToSpawnPostPhysicsInstance(bool bForceReinit) const
{
	if(GetSkeletalMeshAsset())
	{
		const UClass* MainInstanceClass = *AnimClass;
		const UClass* ClassToUse = *GetPostProcessAnimBPClassToBeUsed();
		const UClass* CurrentClass = PostProcessAnimInstance ? PostProcessAnimInstance->GetClass() : nullptr;

		const IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(ClassToUse);
		const USkeleton* AnimSkeleton = (AnimClassInterface) ? AnimClassInterface->GetTargetSkeleton() : nullptr;
		if(AnimSkeleton == nullptr)
		{
			const USkeletalMesh* MeshAsset = GetSkeletalMeshAsset();
			// Fall back to mesh skeleton (post-process anim BP could have been a template)
			AnimSkeleton = MeshAsset ? MeshAsset->GetSkeleton() : nullptr;
		}

		// We need to have an instance, and we have the wrong class (different or null)
		if(ClassToUse && (ClassToUse != CurrentClass || bForceReinit ) && MainInstanceClass != ClassToUse && AnimSkeleton)
		{
			return true;
		}
	}

	return false;
}

bool USkeletalMeshComponent::IsAnimBlueprintInstanced() const
{
	return (AnimScriptInstance && AnimScriptInstance->GetClass() == AnimClass);
}	

bool USkeletalMeshComponent::AreBoneTransformsValid() const
{
	return Super::AreBoneTransformsValid() || bBoneTransformsInitialized;
}

void USkeletalMeshComponent::InitializeBeforeRegister()
{
	// Make sure this is done before we call into the Super which will trigger OnCreatePhysicsState
	UpdateHasValidBodies();
}

void USkeletalMeshComponent::InitializeOnRegister()
{
	checkf(!bBoneTransformsInitialized, TEXT("BoneTransforms should not be initialized before skeletal registration setup."));

	// Ensure we have an empty list of linked instances on registration. Ready for the initialization below 
	// to correctly populate that list.
	ResetLinkedAnimInstances();

	// Ensure the primary tick function is set to tick off the game thread if we are not using internal animation
	PrimaryComponentTick.bRunOnAnyThread = !bEnableAnimation;

	if (bEnableAnimation)
	{
		// We force an initialization here because we're in one of two cases.
		// 1) First register, no spawned instance, need to initialize
		// 2) We're being re-registered, in which case we've went through
		// OnUnregister and unconditionally uninitialized our anim instances
		// so we need to force initialize them before we begin to tick.
		InitAnim(true);
	}
	else
	{
		RecalcRequiredBones(GetPredictedLODLevel());
		ResetToRefPose();
	}

	// Set to true regardless 
	// This handles cases where we don't really update the transforms 
	// - SkeletalMeshAsset is invalid 
	// - LeaderPoseComponent is valid
	// - AnimScriptInstance is valid && InitializeAnimScriptInstance returns false
	bBoneTransformsInitialized = true;
}

void USkeletalMeshComponent::OnRegister()
{
	const bool bHasBeenPreRegistered = IsPreRegistered();

	if (!bHasBeenPreRegistered)
	{
		InitializeBeforeRegister();
	}

	Super::OnRegister();

	if (!bHasBeenPreRegistered)
	{
		InitializeOnRegister();
	}

	if (bRenderStatic || (VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::OnlyTickPoseWhenRendered && !FApp::CanEverRender()))
	{
		SetComponentTickEnabled(false);
	}

	// Create clothing simulations
	RecreateClothingActors();
}

void USkeletalMeshComponent::OnPreRegister()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::OnPreRegister);

	// Temporarily pause the AsyncPhysicsStateJobQueue
	//
	// This is necessary because calling Super::OnPreRegister will schedule an async physics job
	// Pausing new jobs to be added will prevent USkeletalMeshComponent::OnAsyncCreatePhysicsState from being called 
	// until USkeletalMeshComponent::OnPreRegister completes and initializes BoneSpace transforms.
	FPhysScene* PhysScene = GetPhysicsScene();
	check(PhysScene);
	FPhysScene_AsyncPhysicsStateJobQueue::FScopedDeferAsyncPhysicsStateJobs DeferJobs(PhysScene);

	InitializeBeforeRegister();

	Super::OnPreRegister();

	InitializeOnRegister();
}

void USkeletalMeshComponent::OnUnregister()
{
	const bool bBlockOnTask = true; // wait on evaluation task so we complete any work before this component goes away
	const bool bPerformPostAnimEvaluation = false; // Skip post evaluation, it would be wasted work

	// Wait for any in flight animation evaluation to complete
	HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation);

	//clothing actors will be re-created in TickClothing
	ReleaseAllClothingResources();

#if WITH_EDITOR
	OnSkeletalMeshUnregistered.Broadcast(this);
#endif
	
	if (AnimScriptInstance)
	{
		AnimScriptInstance->UninitializeAnimation();
	}

	for(UAnimInstance* LinkedInstance : LinkedInstances)
	{
		LinkedInstance->UninitializeAnimation();
	}
	ResetLinkedAnimInstances();

	if(PostProcessAnimInstance)
	{
		PostProcessAnimInstance->UninitializeAnimation();
	}

	if (DeferredKinematicUpdateIndex != INDEX_NONE)
	{
		UWorld* World = GetWorld();
		FPhysScene* PhysScene = World ? World->GetPhysicsScene() : nullptr;

		if (PhysScene != nullptr)
		{
			PhysScene->ClearPreSimKinematicUpdate(this);
		}
	}

	// Invalidate required bones and our cached data
	RequiredBones.Reset();
	bRequiredBonesUpToDate = false;
	if (SharedRequiredBones)
	{
		SharedRequiredBones->Reset();
	}
	bBoneTransformsInitialized = false;

	Super::OnUnregister();
}

void USkeletalMeshComponent::InitAnim(bool bForceReinit)
{
	CSV_SCOPED_TIMING_STAT(Animation, InitAnim);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SkelMeshComp_InitAnim);
	LLM_SCOPE(ELLMTag::Animation);

	if(!bEnableAnimation)
	{
		return;
	}
	
	// a lot of places just call InitAnim without checking Mesh, so 
	// I'm moving the check here
	if (GetSkeletalMeshAsset() != nullptr && (IsRegistered() || IsPreRegistering()))
	{
		//clear cache UID since we don't know if skeleton changed
		CachedAnimCurveUidVersion = 0;

		// we still need this in case users doesn't call tick, but sent to renderer
		MorphTargetWeights.SetNumZeroed(GetSkeletalMeshAsset()->GetMorphTargets().Num());

		// We may be doing parallel evaluation on the current anim instance
		// Calling this here with true will block this init till that thread completes
		// and it is safe to continue
		const bool bBlockOnTask = true; // wait on evaluation task so it is safe to continue with Init
		const bool bPerformPostAnimEvaluation = true; // That will swap buffer back to ComponentTransform, and finish evaluate. This is required - otherwise, we won't have a buffer.
		HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation);

		bool bBlueprintMismatch = (AnimClass != nullptr) &&
			(AnimScriptInstance != nullptr) && (AnimScriptInstance->GetClass() != AnimClass);

		const USkeleton* AnimSkeleton = (AnimScriptInstance)? AnimScriptInstance->CurrentSkeleton : nullptr;

		const bool bClearAnimInstance = AnimScriptInstance && !AnimSkeleton;
		const bool bSkeletonMismatch = AnimSkeleton && (AnimScriptInstance->CurrentSkeleton!=GetSkeletalMeshAsset()->GetSkeleton());
		const bool bSkeletonsExist = AnimSkeleton && GetSkeletalMeshAsset()->GetSkeleton() && !bSkeletonMismatch;

		LastPoseTickFrame = 0;

		if (bBlueprintMismatch || bSkeletonMismatch || !bSkeletonsExist || bClearAnimInstance)
		{
			ClearAnimScriptInstance();
		}

		// this has to be called before Initialize Animation because it will required RequiredBones list when InitializeAnimScript
		RecalcRequiredBones(GetPredictedLODLevel());

		// In Editor, animations won't get ticked. So Update once to get accurate representation instead of T-Pose.
		// Also allow this to be an option to support pre-4.19 games that might need it..
		const bool bTickAnimationNow =
			(((GetWorld()->WorldType == EWorldType::Editor) && !bForceRefpose)
			|| UAnimationSettings::Get()->bTickAnimationOnSkeletalMeshInit)
			&& !bUseRefPoseOnInitAnim;

		const bool bInitializedAnimInstance = InitializeAnimScriptInstance(bForceReinit, !bTickAnimationNow);

		// Make sure we have a valid pose.
		// We don't allocate transform data when using LeaderPoseComponent, so we have nothing to render.
		if (!LeaderPoseComponent.IsValid())
		{	
			if (bInitializedAnimInstance || (AnimScriptInstance == nullptr))
			{ 
				if (bTickAnimationNow)
				{
					TickAnimation(0.f, false);
					RefreshBoneTransforms();
				}
				else
				{
					ResetToRefPose();
				}

				bBoneTransformsInitialized = true;

				if (bInitializedAnimInstance)
				{
					// Allow blueprints to respond to the event in editor
					FEditorScriptExecutionGuard ScriptGuard;
					OnAnimInitialized.Broadcast();
				}
			}
		}

		UpdateComponentToWorld();
	}
}

#if WITH_EDITOR
void USkeletalMeshComponent::ApplyEditedComponentSpaceTransforms()
{
	// Flip buffers once to copy the directly-written component space transforms
	bNeedToFlipSpaceBaseBuffers = true;
	bHasValidBoneTransform = false;
	FlipEditableSpaceBases();
	bHasValidBoneTransform = true;

	InvalidateCachedBounds();
	UpdateBounds();
	MarkRenderTransformDirty();
	MarkRenderDynamicDataDirty();

	for (auto& FollowerComponent : GetFollowerPoseComponents())
	{
		if (FollowerComponent.IsValid())
		{
			FollowerComponent->UpdateFollowerComponent();
		}
	}
}
#endif

bool USkeletalMeshComponent::InitializeAnimScriptInstance(bool bForceReinit, bool bInDeferRootNodeInitialization)
{
	bool bInitializedMainInstance = false;
	bool bInitializedPostInstance = false;

	USkeletalMesh* SkelMesh = GetSkeletalMeshAsset();
	if (SkelMesh && (IsRegistered() || IsPreRegistering()))
	{
		// We may be doing parallel evaluation on the current anim instance
		// Calling this here with true will block this init till that thread completes
		// and it is safe to continue
		const bool bBlockOnTask = true; // wait on evaluation task so it is safe to continue with Init
		const bool bPerformPostAnimEvaluation = true; // That will swap buffer back to ComponentTransform, and finish evaluate. This is required - otherwise, we won't have a buffer.
		HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation);

		if (NeedToSpawnAnimScriptInstance())
		{
			SCOPE_CYCLE_COUNTER(STAT_AnimSpawnTime);
			AnimScriptInstance = NewObject<UAnimInstance>(this, AnimClass);

			if (AnimScriptInstance)
			{
				// If we have any linked instances left we need to clear them out now, we're about to have a new leader instance
				ResetLinkedAnimInstances();

				AnimScriptInstance->InitializeAnimation(bInDeferRootNodeInitialization);

				// Call BeginPlay on anim instances created at runtime.
				if (HasBegunPlay())
				{
					AnimScriptInstance->NativeBeginPlay();
					AnimScriptInstance->BlueprintBeginPlay();
				}

				bInitializedMainInstance = true;
			}
		}
		else 
		{
			bool bShouldSpawnSingleNodeInstance = SkelMesh && SkelMesh->GetSkeleton() && AnimationMode == EAnimationMode::AnimationSingleNode;
			if (bShouldSpawnSingleNodeInstance)
			{
				SCOPE_CYCLE_COUNTER(STAT_AnimSpawnTime);

				UAnimSingleNodeInstance* OldInstance = nullptr;
				if (!bForceReinit)
				{
					OldInstance = Cast<UAnimSingleNodeInstance>(AnimScriptInstance);
				}

				if (bCVarUninitializeInstanceOnNewSingleNodeInstance && AnimScriptInstance)
				{
					AnimScriptInstance->UninitializeAnimation();
				}

				AnimScriptInstance = NewObject<UAnimSingleNodeInstance>(this);

				if (AnimScriptInstance)
				{
					ResetLinkedAnimInstances();

					AnimScriptInstance->InitializeAnimation(bInDeferRootNodeInitialization);

					if(HasBegunPlay())
					{
						AnimScriptInstance->NativeBeginPlay();
						AnimScriptInstance->BlueprintBeginPlay();
					}

					bInitializedMainInstance = true;
				}

				if (OldInstance && AnimScriptInstance)
				{
					// Copy data from old instance unless we force reinitialized
					FSingleAnimationPlayData CachedData;
					CachedData.PopulateFrom(OldInstance);
					CachedData.Initialize(Cast<UAnimSingleNodeInstance>(AnimScriptInstance));
				}
				else
				{
					// otherwise, initialize with AnimationData
					AnimationData.Initialize(Cast<UAnimSingleNodeInstance>(AnimScriptInstance));
				}

				if (AnimScriptInstance)
				{
					AnimScriptInstance->AddToCluster(this);
				}
			}
		}

		if (AnimScriptInstance && !bInitializedMainInstance && bForceReinit)
		{
			// It is important to make sure that all post-process related stuff happens after a call to this
			// function, otherwise it may reset linked anim instances that the post-process anim instance creates.
			ResetLinkedAnimInstances();

			AnimScriptInstance->InitializeAnimation(bInDeferRootNodeInitialization);
			bInitializedMainInstance = true;
		}

		// May need to clear out the post physics instance
		UClass* NewMeshInstanceClass = *GetPostProcessAnimBPClassToBeUsed();
		if(!NewMeshInstanceClass || NewMeshInstanceClass == *AnimClass || (PostProcessAnimInstance && PostProcessAnimInstance->CurrentSkeleton != GetSkeletalMeshAsset()->GetSkeleton()) || !GetSkeletalMeshAsset()->GetSkeleton())
		{
			PostProcessAnimInstance = nullptr;
		}

		if(NeedToSpawnPostPhysicsInstance(bForceReinit))
		{
			PostProcessAnimInstance = NewObject<UAnimInstance>(this, *GetPostProcessAnimBPClassToBeUsed());

			if(PostProcessAnimInstance)
			{
				PostProcessAnimInstance->InitializeAnimation();

				// Call BeginPlay on anim instances created at runtime.
				if (HasBegunPlay())
				{
					PostProcessAnimInstance->NativeBeginPlay();
					PostProcessAnimInstance->BlueprintBeginPlay();
				}
				
				if(FAnimNode_LinkedInputPose* InputNode = PostProcessAnimInstance->GetLinkedInputPoseNode())
				{
					const FBoneContainer& BoneContainer = PostProcessAnimInstance->GetRequiredBones();
					if (BoneContainer.IsValid())
					{
						InputNode->CachedInputPose.SetBoneContainer(&BoneContainer);
					}

					// SetBoneContainer allocates space for bone data but leaves it uninitialized.
					InputNode->bIsCachedInputPoseInitialized = false;
				}

				bInitializedPostInstance = true;
			}
		}
		else if (!GetPostProcessAnimBPClassToBeUsed())
		{
			PostProcessAnimInstance = nullptr;
		}

		if(PostProcessAnimInstance && !bInitializedPostInstance && bForceReinit)
		{
			PostProcessAnimInstance->InitializeAnimation();
			bInitializedPostInstance = true;
		}

		// refresh morph targets - this can happen when re-registration happens
		RefreshMorphTargets();
	}
	return bInitializedMainInstance || bInitializedPostInstance;
}

bool USkeletalMeshComponent::IsWindEnabled() const
{
	// Wind is enabled in game worlds
	return GetWorld() && GetWorld()->IsGameWorld();
}

void USkeletalMeshComponent::ClearAnimScriptInstance()
{
	if (AnimScriptInstance)
	{
		// We may be doing parallel evaluation on the current anim instance
		// Calling this here with true will block this init till that thread completes
		// and it is safe to continue
		const bool bBlockOnTask = true; // wait on evaluation task so it is safe to swap the buffers
		const bool bPerformPostAnimEvaluation = true; // Do PostEvaluation so we make sure to swap the buffers back. 
		HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation);

		AnimScriptInstance->EndNotifyStates();
		AnimScriptInstance->UninitializeAnimation();
	}
	AnimScriptInstance = nullptr;
	ResetLinkedAnimInstances();
	ClearCachedAnimProperties();
}

void USkeletalMeshComponent::ClearCachedAnimProperties()
{
	CachedBoneSpaceTransforms.Empty();
	CachedComponentSpaceTransforms.Empty();
	CachedCurve.Empty();
	CachedAttributes.Empty();
}

void USkeletalMeshComponent::InitializeComponent()
{
	Super::InitializeComponent();

	InitAnim(false);
}

void USkeletalMeshComponent::BeginPlay()
{
	Super::BeginPlay();

	// Trace the 'first frame' markers
	TRACE_SKELETAL_MESH_COMPONENT(this);

	ForEachAnimInstance([](UAnimInstance* InAnimInstance)
	{
		InAnimInstance->NativeBeginPlay();
		InAnimInstance->BlueprintBeginPlay();
	});
}

#if WITH_EDITOR
FDelegateHandle USkeletalMeshComponent::RegisterOnSkeletalMeshUnregistered(const FOnSkeletalMeshUnregisteredDelegate& Delegate)
{
	return OnSkeletalMeshUnregistered.Add(Delegate);
}

void USkeletalMeshComponent::UnregisterOnSkeletalMeshUnregistered(FDelegateHandle Handle)
{
	OnSkeletalMeshUnregistered.Remove(Handle);
}

void USkeletalMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;

	// Set the skinned asset pointer with the alias pointer (must happen before the call to Super::PostEditChangeProperty)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (PropertyThatChanged && PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, SkeletalMeshAsset))
	{
		SetSkinnedAsset(SkeletalMeshAsset);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::PostEditChangeProperty(PropertyChangedEvent);


	if ( PropertyThatChanged != nullptr )
	{
		// if the blueprint has changed, recreate the AnimInstance
		if ( PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED( USkeletalMeshComponent, AnimationMode ) )
		{
			if (AnimationMode == EAnimationMode::AnimationBlueprint)
			{
				if (AnimClass == nullptr)
				{
					ClearAnimScriptInstance();
				}
				else
				{
					if (NeedToSpawnAnimScriptInstance())
					{
						SCOPE_CYCLE_COUNTER(STAT_AnimSpawnTime);
						AnimScriptInstance = NewObject<UAnimInstance>(this, AnimClass);
						AnimScriptInstance->InitializeAnimation();

						if(HasBegunPlay())
						{
							AnimScriptInstance->NativeBeginPlay();
							AnimScriptInstance->BlueprintBeginPlay();
						}
					}
				}
			}
		}

		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, AnimClass))
		{
			InitAnim(false);
		}

		USkeletalMesh* SkelMesh = GetSkeletalMeshAsset();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if(PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED( USkeletalMeshComponent, SkeletalMeshAsset))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			ValidateAnimation();

			// Check the post physics mesh instance, as the mesh has changed
			if(PostProcessAnimInstance)
			{
				UClass* CurrentClass = PostProcessAnimInstance->GetClass();
				UClass* MeshClass = SkelMesh ? *GetPostProcessAnimBPClassToBeUsed() : nullptr;
				if(CurrentClass != MeshClass)
				{
					if(MeshClass)
					{
						PostProcessAnimInstance = NewObject<UAnimInstance>(this, *GetPostProcessAnimBPClassToBeUsed());
						PostProcessAnimInstance->InitializeAnimation();

						if(HasBegunPlay())
						{
							PostProcessAnimInstance->NativeBeginPlay();
							PostProcessAnimInstance->BlueprintBeginPlay();
						}
					}
					else
					{
						// No instance needed for the new mesh
						PostProcessAnimInstance = nullptr;
					}
				}
			}

			if(OnSkeletalMeshPropertyChanged.IsBound())
			{
				OnSkeletalMeshPropertyChanged.Broadcast();
			}

			// Skeletal mesh was switched so we should clean up the override materials and dirty the render state to recreate material proxies
			if (OverrideMaterials.Num())
			{
				CleanUpOverrideMaterials();
				MarkRenderStateDirty();
			}
			//Make sure the mesh section overlay materials is cleanup and match the new skeletal mesh
			CleanUpMaterialSlotsOverlayMaterial();
		}

		// when user changes simulate physics, just make sure to update blendphysics together
		// bBlendPhysics isn't the editor exposed property, it should work with simulate physics
		if ( PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED( FBodyInstance, bSimulatePhysics ))
		{
			bBlendPhysics = BodyInstance.bSimulatePhysics;
		}

		if ( PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED( FSingleAnimationPlayData, AnimToPlay ))
		{
			// make sure the animation skeleton is valid
			if (AnimationData.AnimToPlay && AnimationData.AnimToPlay->GetSkeleton())
			{
				PlayAnimation(AnimationData.AnimToPlay, false);
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Invalid animation skeleton");
				AnimationData.AnimToPlay = nullptr;
			}
		}

		if ( PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED( FSingleAnimationPlayData, SavedPosition ))
		{
			AnimationData.ValidatePosition();
			SetPosition(AnimationData.SavedPosition, false);
		}

		if ( PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED( USkeletalMeshComponent, TeleportDistanceThreshold ) )
		{
			ClothTeleportDistThresholdSquared = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportDistanceThresholdSquared(TeleportDistanceThreshold);
		}

		if ( PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED( USkeletalMeshComponent, TeleportRotationThreshold ) )
		{
			ClothTeleportCosineThresholdInRad = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportCosineRotationThreshold(TeleportRotationThreshold);
		}
	}
}

void USkeletalMeshComponent::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if(GetLinkerUEVersion() < VER_UE4_REMOVE_SINGLENODEINSTANCE)
	{
		static FName SingleAnimSkeletalComponent_NAME("SingleAnimSkeletalComponent");

		if(OldClassName == SingleAnimSkeletalComponent_NAME)
		{
			SetAnimationMode(EAnimationMode::Type::AnimationSingleNode);

			// support old compatibility code that changed variable name
			if (SequenceToPlay_DEPRECATED!=nullptr && AnimToPlay_DEPRECATED== nullptr)
			{
				AnimToPlay_DEPRECATED = SequenceToPlay_DEPRECATED;
				SequenceToPlay_DEPRECATED = nullptr;
			}

			AnimationData.AnimToPlay = AnimToPlay_DEPRECATED;
			AnimationData.bSavedLooping = bDefaultLooping_DEPRECATED;
			AnimationData.bSavedPlaying = bDefaultPlaying_DEPRECATED;
			AnimationData.SavedPosition = DefaultPosition_DEPRECATED;
			AnimationData.SavedPlayRate = DefaultPlayRate_DEPRECATED;

			MarkPackageDirty();
		}
	}
}

TSoftObjectPtr<UObject> USkeletalMeshComponent::GetDefaultAnimatingRig() const
{
	if (bOverrideDefaultAnimatingRig)
	{
		return DefaultAnimatingRigOverride;
	}
	if (GetSkeletalMeshAsset())
	{
		return GetSkeletalMeshAsset()->GetDefaultAnimatingRig();
	}
	return nullptr;
}

void USkeletalMeshComponent::SetDefaultAnimatingRigOverride(TSoftObjectPtr<UObject> InAnimatingRig) 
{
	DefaultAnimatingRigOverride = InAnimatingRig;
}

TSoftObjectPtr<UObject> USkeletalMeshComponent::GetDefaultAnimatingRigOverride() const
{
	return DefaultAnimatingRigOverride;
}

#endif // WITH_EDITOR




bool USkeletalMeshComponent::ShouldOnlyTickMontages(const float DeltaTime) const
{
	// Ignore DeltaSeconds == 0.f, as that is used when we want to force an update followed by RefreshBoneTransforms.
	// RefreshBoneTransforms will need an updated graph.
	return VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::OnlyTickMontagesWhenNotRendered
		&& !bRecentlyRendered
		&& (DeltaTime > 0.f);
}

bool USkeletalMeshComponent::ShouldOnlyTickMontagesAndRefreshBones(const float DeltaTime) const
{
	// Ignore DeltaSeconds == 0.f, as that is used when we want to force an update followed by RefreshBoneTransforms.
	// RefreshBoneTransforms will need an updated graph.
	return VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::OnlyTickMontagesAndRefreshBonesWhenPlayingMontages
		&& !bRecentlyRendered
		&& (DeltaTime > 0.f)
		&& (GetAnimInstance() && GetAnimInstance()->IsAnyMontagePlaying());
}

void USkeletalMeshComponent::TickAnimation(float DeltaTime, bool bNeedsValidRootMotion)
{
	if(!bEnableAnimation)
	{
		return;
	}

	SCOPED_NAMED_EVENT(USkeletalMeshComponent_TickAnimation, FColor::Yellow);
	SCOPE_CYCLE_COUNTER(STAT_AnimGameThreadTime);
	SCOPE_CYCLE_COUNTER(STAT_AnimTickTime);

	// Recalculate the RequiredBones array, if necessary
	if (!bRequiredBonesUpToDate)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_USkeletalMeshComponent_RefreshBoneTransforms_RecalcRequiredBones);
		RecalcRequiredBones(GetPredictedLODLevel());
	}
	// if curves have to be refreshed
	else if (!AreRequiredCurvesUpToDate())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_USkeletalMeshComponent_RefreshBoneTransforms_RecalcRequiredCurves);
		RecalcRequiredCurves();
	}

	bool bIsCompiling = false;
#if WITH_EDITOR
	bIsCompiling = GetSkeletalMeshAsset() && GetSkeletalMeshAsset()->IsCompiling();
#endif
	if (GetSkeletalMeshAsset() != nullptr && !bIsCompiling)
	{
		// We're about to UpdateAnimation, this will potentially queue events that we'll need to dispatch.
		bNeedsQueuedAnimEventsDispatched = !bSuppressNotifyEventDispatch;

		// Tick all of our anim instances
		TickAnimInstances(DeltaTime, bNeedsValidRootMotion);

		/**
			If we're called directly for autonomous proxies, TickComponent is not guaranteed to get called.
			So dispatch all queued events here if we're doing MontageOnly ticking.
		*/
		if (ShouldOnlyTickMontages(DeltaTime) || ShouldOnlyTickMontagesAndRefreshBones(DeltaTime))
		{
			ConditionallyDispatchQueuedAnimEvents();
		}
	}
}

void USkeletalMeshComponent::SetPredictedLODLevel(int32 InPredictedLODLevel)
{
	int32 OldPredictedLODLevel = GetPredictedLODLevel();
	
	Super::SetPredictedLODLevel(InPredictedLODLevel);

	if(OldPredictedLODLevel != GetPredictedLODLevel())
	{
		bRequiredBonesUpToDate = false;
	}
} 

void USkeletalMeshComponent::TickAnimInstances(float DeltaTime, bool bNeedsValidRootMotion)
{
	if(!bEnableAnimation)
	{
		return;
	}

	// Allow animation instance to do some processing before the linked instances update
	if (AnimScriptInstance != nullptr)
	{
		AnimScriptInstance->PreUpdateLinkedInstances(DeltaTime * GlobalAnimRateScale);
	}

	// We update linked instances first incase we're using either root motion or non-threaded update.
	// This ensures that we go through the pre update process and initialize the proxies correctly.
	for (UAnimInstance* LinkedInstance : LinkedInstances)
	{
		// Sub anim instances are always forced to do a parallel update 
		LinkedInstance->UpdateAnimation(DeltaTime * GlobalAnimRateScale, false, UAnimInstance::EUpdateAnimationFlag::ForceParallelUpdate);
	}

	if (AnimScriptInstance != nullptr)
	{
		// Tick the animation
		AnimScriptInstance->UpdateAnimation(DeltaTime * GlobalAnimRateScale, bNeedsValidRootMotion);
	}

	if(ShouldUpdatePostProcessInstance())
	{
		PostProcessAnimInstance->UpdateAnimation(DeltaTime * GlobalAnimRateScale, false);
	}
}

void USkeletalMeshComponent::TickMontagesOnly(float DeltaTime, bool bNeedsValidRootMotion)
{
	if (!bEnableAnimation)
	{
		if (AnimScriptInstance)
		{
			if (!ShouldTickPose())
			{
				return;
			}

			// Update the pose tick frame counter to avoid double ticking the montages. 
			LastPoseTickFrame = static_cast<uint32>(GFrameCounter);

			AnimScriptInstance->TickMontageOnly(DeltaTime);
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "SkeletalMeshComponent in '%ls' tried to TickMontagesOnly but there is no valid AnimInstance", *GetNameSafe(GetOwner()));
		}
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "SkeletalMeshComponent in '%ls' tried to call TickMontagesOnly when animations are enabled. This is likely an error and will be skipped.", *GetNameSafe(GetOwner()));
	}
}

bool USkeletalMeshComponent::UpdateLODStatus()
{
	if (Super::UpdateLODStatus())
	{
		bRequiredBonesUpToDate = false;
		bValidMeshPoseTransformsUpToDate = false;
		return true;
	}

	return false;
}

void USkeletalMeshComponent::UpdateVisualizeLODString(FString& DebugString)
{
	Super::UpdateVisualizeLODString(DebugString);

	uint32 NumVertices = 0;
	if (GetSkeletalMeshAsset())
	{
		if (FSkeletalMeshRenderData* RenderData = GetSkeletalMeshAsset()->GetResourceForRendering())
		{
			if (RenderData->LODRenderData.IsValidIndex(GetPredictedLODLevel()))
			{
				NumVertices = RenderData->LODRenderData[GetPredictedLODLevel()].GetNumVertices();
			}
		}
	}

	DebugString = DebugString + FString::Printf(TEXT("\nRequiredBones(%d) NumVerts(%d)"), 
		RequiredBones.Num(), NumVertices);
}

bool USkeletalMeshComponent::ShouldUpdateTransform(bool bLODHasChanged) const
{
#if WITH_EDITOR	
	// If we're in an editor world (Non running, WorldType will be PIE when simulating or in PIE) then we only want transform updates on LOD changes as the
	// animation isn't running so it would just waste CPU time
	if(GetWorld()->WorldType == EWorldType::Editor)
	{
		if( bUpdateAnimationInEditor )
		{
			return true;
		}

		// if leader pose is ticking, follower also has to update it
		if (LeaderPoseComponent.IsValid())
		{
			const USkeletalMeshComponent* Leader = Cast<USkeletalMeshComponent>(LeaderPoseComponent.Get());
			if (Leader && Leader->GetUpdateAnimationInEditor())
			{
				return true;
			}
		}

		return bLODHasChanged;
	}
#endif

	// If forcing RefPose we can skip updating the skeleton for perf, except if it's using MorphTargets.
	const bool bSkipBecauseOfRefPose = bForceRefpose && bOldForceRefPose && (MorphTargetCurves.Num() == 0) && ((AnimScriptInstance) ? !AnimScriptInstance->HasMorphTargetCurves() : true);

	const bool bShouldUpdateTransform = Super::ShouldUpdateTransform(bLODHasChanged) ||
			(GetAnimInstance() && GetAnimInstance()->IsAnyMontagePlaying()
			&& VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::OnlyTickMontagesAndRefreshBonesWhenPlayingMontages);

	return (bShouldUpdateTransform && !bNoSkeletonUpdate && !bSkipBecauseOfRefPose);
}

bool USkeletalMeshComponent::ShouldTickPose() const
{
	if (LeaderPoseComponent.IsValid() && !bFollowerShouldTickPose)
	{
		return false;
	}

	// When we stop root motion we go back to ticking after CharacterMovement. Unfortunately that means that we could tick twice that frame.
	// So only enforce a single tick per frame.
	const bool bAlreadyTickedThisFrame = PoseTickedThisFrame();

#if WITH_EDITOR
	if (GetWorld()->WorldType == EWorldType::Editor)
	{
		if (bUpdateAnimationInEditor)
		{
			return true;
		}
	}
#endif 

	// Allow tick pose if there is any montage playing and has corresponding visibility based tick option while not rendered
	const bool bShouldTickBasedOnMontage = (VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::OnlyTickMontagesWhenNotRendered || VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::OnlyTickMontagesAndRefreshBonesWhenPlayingMontages)
		&& !bRecentlyRendered
		&& (GetAnimInstance() && GetAnimInstance()->IsAnyMontagePlaying());

	// Autonomous Ticking is allowed to occur multiple times per frame, as we can receive and process multiple networking updates the same frame.
	const bool bShouldTickBasedOnAutonomousCheck = bIsAutonomousTickPose || (!bOnlyAllowAutonomousTickPose && !bAlreadyTickedThisFrame);
	// When playing networked Root Motion Montages, we want these to play on dedicated servers and remote clients for networking and position correction purposes.
	// So we force pose updates in that case to keep root motion and position in sync.
	const bool bShouldTickBasedOnVisibility = (bShouldTickBasedOnMontage || (VisibilityBasedAnimTickOption <= EVisibilityBasedAnimTickOption::AlwaysTickPose) || bRecentlyRendered || IsPlayingNetworkedRootMotionMontage());

	return (bShouldTickBasedOnVisibility && bShouldTickBasedOnAutonomousCheck && IsRegistered() && (AnimScriptInstance || PostProcessAnimInstance) && !bPauseAnims && GetWorld()->AreActorsInitialized() && !bNoSkeletonUpdate);
}

bool USkeletalMeshComponent::ShouldTickAnimation() const
{
	if(bExternalTickRateControlled)
	{
		return bExternalUpdate;
	}
	else
	{
		return (AnimUpdateRateParams != nullptr) && (!ShouldUseUpdateRateOptimizations() || !AnimUpdateRateParams->ShouldSkipUpdate());
	}
}

static FThreadSafeCounter Ticked;
static FThreadSafeCounter NotTicked;

static TAutoConsoleVariable<int32> CVarSpewAnimRateOptimization(
	TEXT("SpewAnimRateOptimization"),
	0,
	TEXT("True to spew overall anim rate optimization tick rates."));

void USkeletalMeshComponent::TickPose(float DeltaTime, bool bNeedsValidRootMotion)
{
	Super::TickPose(DeltaTime, bNeedsValidRootMotion);

	if (ShouldTickAnimation())
	{
		// Don't care about roll over, just care about uniqueness (and 32-bits should give plenty).
		LastPoseTickFrame = static_cast<uint32>(GFrameCounter);

		float DeltaTimeForTick;
		if(bExternalTickRateControlled)
		{
			DeltaTimeForTick = ExternalDeltaTime;
		}
		else if(ShouldUseUpdateRateOptimizations())
		{
			DeltaTimeForTick = DeltaTime + AnimUpdateRateParams->GetTimeAdjustment();
		}
		else
		{
			DeltaTimeForTick = DeltaTime;
		}

		TickAnimation(DeltaTimeForTick, bNeedsValidRootMotion);
		if (CVarSpewAnimRateOptimization.GetValueOnGameThread() > 0 && Ticked.Increment()==500)
		{
			UE_LOGF(LogTemp, Display, "%d Ticked %d NotTicked", Ticked.GetValue(), NotTicked.GetValue());
			Ticked.Reset();
			NotTicked.Reset();
		}
	}
	else if(!bExternalTickRateControlled)
	{
		if (AnimScriptInstance)
		{
			AnimScriptInstance->OnUROSkipTickAnimation();
		}

		for(UAnimInstance* LinkedInstance : LinkedInstances)
		{
			LinkedInstance->OnUROSkipTickAnimation();
		}

		if(PostProcessAnimInstance)
		{
			PostProcessAnimInstance->OnUROSkipTickAnimation();
		}

		if (CVarSpewAnimRateOptimization.GetValueOnGameThread())
		{
			NotTicked.Increment();
		}
	}
}

void USkeletalMeshComponent::ResetMorphTargetCurves()
{
	ActiveMorphTargets.Reset();

	if (GetSkeletalMeshAsset())
	{
		MorphTargetWeights.SetNum(GetSkeletalMeshAsset()->GetMorphTargets().Num());

		// we need this code to ensure the buffer gets cleared whether or not you have morphtarget curve set
		// the case, where you had morphtargets weight on, and when you clear the weight, you want to make sure 
		// the buffer gets cleared and resized
		if (MorphTargetWeights.Num() > 0)
		{
			FMemory::Memzero(MorphTargetWeights.GetData(), MorphTargetWeights.GetAllocatedSize());
		}
	}
	else
	{
		MorphTargetWeights.Reset();
	}
}

void USkeletalMeshComponent::UpdateMorphTargetOverrideCurves()
{
	if (GetSkeletalMeshAsset())
	{
		if (MorphTargetCurves.Num() > 0)
		{
			FAnimationRuntime::AppendActiveMorphTargets(GetSkeletalMeshAsset(), MorphTargetCurves, ActiveMorphTargets, MorphTargetWeights);
		}
	}
}

static TAutoConsoleVariable<int32> CVarAnimationDelaysEndGroup(
	TEXT("tick.AnimationDelaysEndGroup"),
	1,
	TEXT("If > 0, then skeletal meshes that do not rely on physics simulation will set their animation end tick group to TG_PostPhysics."));

void USkeletalMeshComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Animation);

#if WITH_EDITOR
	//Do not tick skeletal mesh component if the skeletal mesh asset is compiling
	if (GetSkeletalMeshAsset() && GetSkeletalMeshAsset()->IsCompiling())
	{
		return;
	}
#endif

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if(!PrimaryComponentTick.bRunOnAnyThread)
	{
		// Cloth work must be done after Super::TickComponent so that it has the correct PredictedLOD

		UpdateEndPhysicsTickRegisteredState();
		UpdateClothTickRegisteredState();
		if(!ClothTickFunction.IsTickFunctionRegistered())
		{
			// Do any cloth work that needs to be done when it doesn't tick.
			CSV_SCOPED_TIMING_STAT(Animation, Cloth);
			const int32 PredictedLOD = GetPredictedLODLevel();

			// If we are suspended, we will not simulate clothing, but as clothing is simulated in local space
			// relative to a root bone we need to extract simulation positions as this bone could be animated.
			if (!CVarEnableClothPhysics.GetValueOnGameThread() || bClothingSimulationSuspended)
			{
				for (FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
				{
					// First update the simulation context, since the simulation isn't ticking
					// and it is still required to get the correct simulation data and bounds.
					constexpr bool bIsInitialization = false;
					ClothingSimulationInstance.FillContextAndPrepareTick(this, DeltaTime, bIsInitialization);
				}

				WritebackClothingSimulationData();
			}
			else
			{
				// Since cloth isn't ticking, force a teleport and reset the next time it does.
				ForceClothNextUpdateTeleportAndReset();
			}
		}

		PendingRadialForces.Reset();

		// Update bOldForceRefPose
		bOldForceRefPose = bForceRefpose;

		/** Update the end group and tick priority */
		const bool bDoLateEnd = CVarAnimationDelaysEndGroup.GetValueOnGameThread() > 0;
		const bool bRequiresPhysics = EndPhysicsTickFunction.IsTickFunctionRegistered();
		const ETickingGroup EndTickGroup = bDoLateEnd && !bRequiresPhysics ? TG_PostPhysics : TG_PrePhysics;
		if (ThisTickFunction)
		{
			ThisTickFunction->EndTickGroup = EndTickGroup;

			const bool bDoHiPri = CVarHiPriSkinnedMeshesTicks.GetValueOnGameThread() > 0;
			check(PrimaryComponentTick.bHighPriority == bDoHiPri)
		}

		// If we are waiting for ParallelEval to complete or if we require Physics, 
		// then FinalizeBoneTransform will be called and Anim events will be dispatched there. 
		// We prefer doing it there so these events are triggered once we have a new updated pose.
		// Note that it's possible that FinalizeBoneTransform has already been called here if not using ParallelUpdate.
		// or it's possible that it hasn't been called at all if we're skipping Evaluate due to not being visible.
		// ConditionallyDispatchQueuedAnimEvents will catch that and only Dispatch events if not already done.
		if (!IsRunningParallelEvaluation() && !bRequiresPhysics)
		{
			/////////////////////////////////////////////////////////////////////////////
			// Notify / Event Handling!
			// This can do anything to our component (including destroy it) 
			// Any code added after this point needs to take that into account
			/////////////////////////////////////////////////////////////////////////////

			ConditionallyDispatchQueuedAnimEvents();
		}
	}
}

void USkeletalMeshComponent::ConditionallyDispatchQueuedAnimEvents()
{
	if (bNeedsQueuedAnimEventsDispatched)
	{
		bNeedsQueuedAnimEventsDispatched = false;

		// Copy linked instances here, as anim notifies could potentially modify this array
		for (UAnimInstance* LinkedInstance : TArray<TObjectPtr<UAnimInstance>, TInlineAllocator<8>>(LinkedInstances))
		{
			LinkedInstance->DispatchQueuedAnimEvents();
		}

		if (AnimScriptInstance)
		{
			AnimScriptInstance->DispatchQueuedAnimEvents();
		}

		if (PostProcessAnimInstance)
		{
			PostProcessAnimInstance->DispatchQueuedAnimEvents();
		}
	}
}

/** 
 *	Utility for taking two arrays of bone indices, which must be strictly increasing, and finding the intersection between them.
 *	That is - any item in the output should be present in both A and B. Output is strictly increasing as well.
 */
static void IntersectBoneIndexArrays(TArray<FBoneIndexType>& Output, const TArray<FBoneIndexType>& A, const TArray<FBoneIndexType>& B)
{
	int32 APos = 0;
	int32 BPos = 0;
	while(	APos < A.Num() && BPos < B.Num() )
	{
		// If value at APos is lower, increment APos.
		if( A[APos] < B[BPos] )
		{
			APos++;
		}
		// If value at BPos is lower, increment APos.
		else if( B[BPos] < A[APos] )
		{
			BPos++;
		}
		// If they are the same, put value into output, and increment both.
		else
		{
			Output.Add( A[APos] );
			APos++;
			BPos++;
		}
	}
}

// this is optimized version of updating only curves
// if you call RecalcRequiredBones, curve should be refreshed
void USkeletalMeshComponent::RecalcRequiredCurves()
{
	if (!GetSkeletalMeshAsset())
	{
		return;
	}

	UE::Anim::FCurveFilterSettings CurveFilterSettings = GetCurveFilterSettings();

	// make sure animation requiredcurve to mark as dirty
	if (AnimScriptInstance)
	{
		AnimScriptInstance->RecalcRequiredCurves(CurveFilterSettings);
	}

	for(UAnimInstance* LinkedInstance : LinkedInstances)
	{
		LinkedInstance->RecalcRequiredCurves(CurveFilterSettings);
	}

	if(PostProcessAnimInstance)
	{
		PostProcessAnimInstance->RecalcRequiredCurves(CurveFilterSettings);
	}

	MarkRequiredCurveUpToDate();
}

void USkeletalMeshComponent::ComputeRequiredBones(TArray<FBoneIndexType>& OutRequiredBones, TArray<FBoneIndexType>& OutFillComponentSpaceTransformsRequiredBones, int32 LODIndex, bool bIgnorePhysicsAsset) const
{
	OutRequiredBones.Reset();
	OutFillComponentSpaceTransformsRequiredBones.Reset();

	USkeletalMesh* SkelMesh = GetSkeletalMeshAsset();
	if (!SkelMesh)
	{
		return;
	}

	FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();
	if (!SkelMeshRenderData)
	{
		//No Render Data?
		// Jira UE-64409
		UE_LOGF(LogAnimation, Warning, "Skeletal Mesh asset '%ls' has no render data", *SkelMesh->GetName());
		return;
	}

	// Make sure we access a valid LOD
	// @fixme jira UE-30028 Avoid crash when called with partially loaded asset
	if (SkelMeshRenderData->LODRenderData.Num() == 0)
	{
		//No LODS?
		UE_LOGF(LogAnimation, Warning, "Skeletal Mesh asset '%ls' has no LODs", *SkelMesh->GetName());
		return;
	}

	LODIndex = FMath::Clamp(LODIndex, 0, SkelMeshRenderData->LODRenderData.Num() - 1);

	// The list of bones we want is taken from the predicted LOD level.
	FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];
	OutRequiredBones = LODData.RequiredBones;

	// Add virtual bones
	GetRequiredVirtualBones(SkelMesh, OutRequiredBones);

	const UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	// If we have a PhysicsAsset, we also need to make sure that all the bones used by it are always updated, as its used
	// by line checks etc. We might also want to kick in the physics, which means having valid bone transforms.
	if (!bIgnorePhysicsAsset && PhysicsAsset)
	{
		GetPhysicsRequiredBones(SkelMesh, PhysicsAsset, OutRequiredBones);
	}

	// TODO - Make sure that bones with per-poly collision are also always updated.

	// Get any additional required bones from followers
	for (const TWeakObjectPtr<USkinnedMeshComponent>& FollowerPoseComponent : FollowerPoseComponents)
	{
		if (const USkinnedMeshComponent* const FollowerPoseComponentPtr = FollowerPoseComponent.Get())
		{
			FollowerPoseComponentPtr->GetAdditionalRequiredBonesForLeader(LODIndex, OutRequiredBones);
		}
	}

	// Purge invisible bones and their children
	// this has to be done before mirror table check/physics body checks
	// mirror table/phys body ones has to be calculated
	ExcludeHiddenBones(this, SkelMesh, OutRequiredBones);

	// Get socket bones set to animate and bones required to fill the component space base transforms
	TArray<FBoneIndexType> NeededBonesForFillComponentSpaceTransforms;
	GetSocketRequiredBones(SkelMesh, OutRequiredBones, NeededBonesForFillComponentSpaceTransforms);

	// Gather any bones referenced by shadow shapes
	GetShadowShapeRequiredBones(this, OutRequiredBones);

	// Ensure that we have a complete hierarchy down to those bones.
	FAnimationRuntime::EnsureParentsPresent(OutRequiredBones, SkelMesh->GetRefSkeleton());
	OutRequiredBones.Sort();

	OutFillComponentSpaceTransformsRequiredBones.Reset(OutRequiredBones.Num() + NeededBonesForFillComponentSpaceTransforms.Num());
	OutFillComponentSpaceTransformsRequiredBones = OutRequiredBones;

	NeededBonesForFillComponentSpaceTransforms.Sort();
	MergeInBoneIndexArrays(OutFillComponentSpaceTransformsRequiredBones, NeededBonesForFillComponentSpaceTransforms);
	FAnimationRuntime::EnsureParentsPresent(OutFillComponentSpaceTransformsRequiredBones, SkelMesh->GetRefSkeleton());
}

void USkeletalMeshComponent::ComputeValidMeshPoseTransforms(TArray<FBoneIndexType>& InRequiredBones, TBitArray<>& OutValidTransforms) const
{
	OutValidTransforms.Init(false, GetNumComponentSpaceTransforms());
	for (int32 i = 0; i < InRequiredBones.Num(); ++i)
	{
		OutValidTransforms[InRequiredBones[i]] = true; 
	}
}

/*static*/ void USkeletalMeshComponent::GetRequiredVirtualBones(const USkeletalMesh* SkeletalMesh, TArray<FBoneIndexType>& OutRequiredBones)
{
	check (SkeletalMesh != nullptr);

	// Add virtual bones
	MergeInBoneIndexArrays(OutRequiredBones, SkeletalMesh->GetRefSkeleton().GetRequiredVirtualBones());
}

/*static*/ void USkeletalMeshComponent::ExcludeHiddenBones(const USkeletalMeshComponent* SkeletalMeshComponent, const USkeletalMesh* SkeletalMesh, TArray<FBoneIndexType>& OutRequiredBones)
{
	check(SkeletalMeshComponent != nullptr);
	check(SkeletalMesh != nullptr);

	const TArray<uint8>& EditableBoneVisibilityStates = SkeletalMeshComponent->GetEditableBoneVisibilityStates();
	if (SkeletalMeshComponent->ShouldUpdateBoneVisibility() && EditableBoneVisibilityStates.Num() > 0)
	{
		if (!ensureMsgf(EditableBoneVisibilityStates.Num() == SkeletalMeshComponent->GetNumComponentSpaceTransforms(),
		TEXT("Skeletal Mesh asset '%s' has incorrect BoneVisibilityStates. # of BoneVisibilityStates (%d), # of SkeletalMeshComponent->GetNumComponentSpaceTransforms() (%d)"),
		*SkeletalMesh->GetName(), EditableBoneVisibilityStates.Num(), SkeletalMeshComponent->GetNumComponentSpaceTransforms()))
		{
			return;
		}

		if (ensureMsgf(EditableBoneVisibilityStates.Num() >= OutRequiredBones.Num(),
			TEXT("Skeletal Mesh asset '%s' has incorrect BoneVisibilityStates. # of BoneVisibilityStates (%d), # of OutRequiredBones (%d)"),
			*SkeletalMesh->GetName(), EditableBoneVisibilityStates.Num(), OutRequiredBones.Num()))
		{
			int32 VisibleBoneWriteIndex = 0;
			for (int32 i = 0; i < OutRequiredBones.Num(); ++i)
			{
				FBoneIndexType CurBoneIndex = OutRequiredBones[i];

				// Current bone visible?
				if (EditableBoneVisibilityStates[CurBoneIndex] == BVS_Visible ||
					SkeletalMeshComponent->VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones)
				{
					OutRequiredBones[VisibleBoneWriteIndex++] = CurBoneIndex;
				}
			}

			// Remove any trailing junk in the OutRequiredBones array
			const int32 NumBonesHidden = OutRequiredBones.Num() - VisibleBoneWriteIndex;
			if (NumBonesHidden > 0)
			{
				OutRequiredBones.RemoveAt(VisibleBoneWriteIndex, NumBonesHidden);
			}
		}
	}
}

/*static*/ void USkeletalMeshComponent::GetMirroringRequiredBones(const USkeletalMesh* SkeletalMesh, TArray<FBoneIndexType>& OutRequiredBones)
{
}

/*static*/ void USkeletalMeshComponent::GetShadowShapeRequiredBones(const USkeletalMeshComponent* SkeletalMeshComponent, TArray<FBoneIndexType>& OutRequiredBones)
{
	if (SkeletalMeshComponent->SceneProxy && !SkeletalMeshComponent->SceneProxy->IsNaniteMesh()) // TODO: Nanite-Skinning
	{
		FSkeletalMeshSceneProxy* SkeletalMeshProxy = (FSkeletalMeshSceneProxy*)SkeletalMeshComponent->SceneProxy;
		const TArray<FBoneIndexType>& ShadowShapeBones = SkeletalMeshProxy->GetSortedShadowBoneIndices();

		if (ShadowShapeBones.Num())
		{
			// Sort in hierarchy order then merge to required bones array
			MergeInBoneIndexArrays(OutRequiredBones, ShadowShapeBones);
		}
	}
}


void USkeletalMeshComponent::RecalcRequiredBones(int32 LODIndex)
{
	if (!GetSkeletalMeshAsset())
	{
		return;
	}

	ComputeRequiredBones(RequiredBones, FillComponentSpaceTransformsRequiredBones, LODIndex, /*bIgnorePhysicsAsset=*/ false);

	// Reset our animated pose to the reference pose
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BoneSpaceTransforms = GetSkeletalMeshAsset()->GetRefSkeleton().GetRefBonePose();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Make sure no other parallel task is ongoing since we need to reset the shared required bones
	// and they might be in use
	HandleExistingParallelEvaluationTask(true, false);

	// If we had cached our shared bone container, reset it
	if (SharedRequiredBones)
	{
		SharedRequiredBones->Reset();
	}

	// make sure animation requiredBone to mark as dirty
	if (AnimScriptInstance)
	{
		AnimScriptInstance->RecalcRequiredBones();
	}

	for (UAnimInstance* LinkedInstance : LinkedInstances)
	{
		LinkedInstance->RecalcRequiredBones();
	}

	if (PostProcessAnimInstance)
	{
		PostProcessAnimInstance->RecalcRequiredBones();
	}

	// when RecalcRequiredBones happened
	// this should always happen
	MarkRequiredCurveUpToDate();
	bRequiredBonesUpToDate = true;

	// Invalidate cached bones.
	ClearCachedAnimProperties();

	OnLODRequiredBonesUpdate_Static.Broadcast(this, LODIndex, RequiredBones);
	OnLODRequiredBonesUpdate_Member.Broadcast(this, LODIndex, RequiredBones);
}

void USkeletalMeshComponent::MarkRequiredCurveUpToDate()
{
	if(USkeletalMesh* Mesh = GetSkeletalMeshAsset())
	{
		if(USkeleton* Skeleton = Mesh->GetSkeleton())
		{
			CachedAnimCurveUidVersion = GetSkeletalMeshAsset()->GetSkeleton()->GetAnimCurveUidVersion();
		}

		if(UAnimCurveMetaData* AnimCurveMetaData = Mesh->GetAssetUserData<UAnimCurveMetaData>())
		{
			CachedMeshCurveMetaDataVersion = AnimCurveMetaData->GetVersionNumber();
		}
		else
		{
			CachedMeshCurveMetaDataVersion = 0;
		}
	}
}

bool USkeletalMeshComponent::AreRequiredCurvesUpToDate() const
{
	if(USkeletalMesh* Mesh = GetSkeletalMeshAsset())
	{
		uint16 MeshVersion = 0;
		if(UAnimCurveMetaData* AnimCurveMetaData = Mesh->GetAssetUserData<UAnimCurveMetaData>())
		{
			MeshVersion = AnimCurveMetaData->GetVersionNumber();
		}

		uint16 SkeletonVersion = 0;
		if(USkeleton* Skeleton = Mesh->GetSkeleton())
		{
			SkeletonVersion = Skeleton->GetAnimCurveUidVersion();
		}

		return CachedAnimCurveUidVersion == SkeletonVersion && CachedMeshCurveMetaDataVersion == MeshVersion;
	}

	return true;
}

void USkeletalMeshComponent::EvaluateAnimation(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, bool bInForceRefPose, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve, FCompactPose& OutPose, UE::Anim::FHeapAttributeContainer& OutAttributes) const
{
	ANIM_MT_SCOPE_CYCLE_COUNTER(SkeletalComponentAnimEvaluate, !IsInGameThread());

	if( !InSkeletalMesh )
	{
		return;
	}

	// We can only evaluate animation if RequiredBones is properly setup for the right mesh!
	if( InSkeletalMesh->GetSkeleton() && 
		InAnimInstance &&
		InAnimInstance->ParallelCanEvaluate(InSkeletalMesh))
	{
		UE_ADD_CRASH_CONTEXT_SCOPE([&](FCrashContextExtendedWriter& Writer)
		{
			const FString Context = FString::Printf(
				TEXT("SkeletalMeshComponent='%s' SkeletalMesh='%s' Skeleton='%s' AnimInstance='%s' bForceRefPose=%s"),
				*GetNameSafe(this),
				*GetNameSafe(InSkeletalMesh),
				*GetNameSafe(InSkeletalMesh ? InSkeletalMesh->GetSkeleton() : nullptr),
				*GetNameSafe(InAnimInstance),
				bInForceRefPose ? TEXT("true") : TEXT("false")
			);
			Writer.AddString(TEXT("AnimationContext"), *Context);
		});

		FParallelEvaluationData EvaluationData = { OutCurve, OutPose, OutAttributes };
		InAnimInstance->ParallelEvaluateAnimation(bInForceRefPose, InSkeletalMesh, EvaluationData);
	}
}

void USkeletalMeshComponent::UpdateFollowerComponent()
{
	ResetMorphTargetCurves();

	if(ensure(LeaderPoseComponent.IsValid()))
	{
		if (USkeletalMeshComponent* LeaderSMC = Cast<USkeletalMeshComponent>(LeaderPoseComponent.Get()))
		{
			// first set any animation-driven curves from the leader SMC
			if (LeaderSMC->AnimScriptInstance)
			{
				LeaderSMC->AnimScriptInstance->RefreshCurves(this);
			}

			// we changed order of morphtarget to be overriden by SetMorphTarget from BP
			// so this has to go first
			// now propagate BP-driven curves from the leader SMC...
			if (GetSkeletalMeshAsset())
			{
				check(MorphTargetWeights.Num() == GetSkeletalMeshAsset()->GetMorphTargets().Num());
				if (LeaderSMC->MorphTargetCurves.Num() > 0)
				{
					FAnimationRuntime::AppendActiveMorphTargets(GetSkeletalMeshAsset(), LeaderSMC->MorphTargetCurves, ActiveMorphTargets, MorphTargetWeights);
				}

				// if follower also has it, add it here. 
				if (MorphTargetCurves.Num() > 0)
				{
					FAnimationRuntime::AppendActiveMorphTargets(GetSkeletalMeshAsset(), MorphTargetCurves, ActiveMorphTargets, MorphTargetWeights);
				}
			}
		}
	}
 
	Super::UpdateFollowerComponent();
}

#if WITH_EDITOR

void USkeletalMeshComponent::PerformAnimationEvaluation(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, TArray<FTransform>& OutSpaceBases, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve, UE::Anim::FMeshAttributeContainer& OutAttributes)
{
	PerformAnimationProcessing(InSkeletalMesh, InAnimInstance, true, false, OutSpaceBases, OutBoneSpaceTransforms, OutRootBoneTranslation, OutCurve, OutAttributes);
}

void USkeletalMeshComponent::PerformAnimationEvaluation(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, TArray<FTransform>& OutSpaceBases, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve)
{
	UE::Anim::FMeshAttributeContainer Attributes;
	PerformAnimationEvaluation(InSkeletalMesh, InAnimInstance, OutSpaceBases, OutBoneSpaceTransforms, OutRootBoneTranslation, OutCurve, Attributes);
}

#endif

void USkeletalMeshComponent::PerformAnimationProcessing(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, bool bInDoEvaluation, bool bInForceRefPose, TArray<FTransform>& OutSpaceBases, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve, UE::Anim::FMeshAttributeContainer& OutAttributes)
{
	CSV_SCOPED_TIMING_STAT(Animation, WorkerThreadTickTime);
	ANIM_MT_SCOPE_CYCLE_COUNTER(PerformAnimEvaluation, !IsInGameThread());

	// Can't do anything without a SkeletalMesh
	if (!InSkeletalMesh)
	{
		return;
	}

	// update anim instance
	if(InAnimInstance && InAnimInstance->NeedsUpdate())
	{
		InAnimInstance->ParallelUpdateAnimation();
	}
	
	if(ShouldPostUpdatePostProcessInstance())
	{
		// If we don't have an anim instance, we may still have a post physics instance
		PostProcessAnimInstance->ParallelUpdateAnimation();
	}

	// Do nothing more if no bones in skeleton.
	if(bInDoEvaluation && OutSpaceBases.Num() > 0)
	{
		FMemMark Mark(FMemStack::Get());
		FCompactPose EvaluatedPose;

		UE::Anim::FHeapAttributeContainer Attributes;		

		// evaluate pure animations, and fill up BoneSpaceTransforms
		EvaluateAnimation(InSkeletalMesh, InAnimInstance, bInForceRefPose, OutRootBoneTranslation, OutCurve, EvaluatedPose, Attributes);
		EvaluatePostProcessMeshInstance(OutBoneSpaceTransforms, EvaluatedPose, OutCurve, InSkeletalMesh, OutRootBoneTranslation, Attributes, bInForceRefPose);

		// Finalize the transforms from the evaluation
		FinalizePoseEvaluationResult(InSkeletalMesh, OutBoneSpaceTransforms, OutRootBoneTranslation, EvaluatedPose);

		if (EvaluatedPose.IsValid())
		{
			FinalizeAttributeEvaluationResults(EvaluatedPose.GetBoneContainer(), Attributes, OutAttributes);
		}

		// Fill SpaceBases from LocalAtoms
		InSkeletalMesh->FillComponentSpaceTransforms(OutBoneSpaceTransforms, FillComponentSpaceTransformsRequiredBones, OutSpaceBases);
	}
}


void USkeletalMeshComponent::PerformAnimationProcessing(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, bool bInDoEvaluation, TArray<FTransform>& OutSpaceBases, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve, UE::Anim::FMeshAttributeContainer& OutAttributes)
{
	PerformAnimationProcessing(InSkeletalMesh, InAnimInstance, bInDoEvaluation, false, OutSpaceBases, OutBoneSpaceTransforms, OutRootBoneTranslation, OutCurve, OutAttributes);
}

void USkeletalMeshComponent::EvaluatePostProcessMeshInstance(TArray<FTransform>& OutBoneSpaceTransforms, FCompactPose& InOutPose, FBlendedHeapCurve& OutCurve, const USkeletalMesh* InSkeletalMesh, FVector& OutRootBoneTranslation, UE::Anim::FHeapAttributeContainer& OutAttributes) const
{
	EvaluatePostProcessMeshInstance(OutBoneSpaceTransforms, InOutPose, OutCurve, InSkeletalMesh, OutRootBoneTranslation, OutAttributes, false);
}

void USkeletalMeshComponent::EvaluatePostProcessMeshInstance(TArray<FTransform>& OutBoneSpaceTransforms, FCompactPose& InOutPose, FBlendedHeapCurve& OutCurve, const USkeletalMesh* InSkeletalMesh, FVector& OutRootBoneTranslation, UE::Anim::FHeapAttributeContainer& OutAttributes, bool bInForceRefPose) const
{
	if (ShouldEvaluatePostProcessInstance())
	{
		// Push the previous pose to any input nodes required
		if (FAnimNode_LinkedInputPose* InputNode = PostProcessAnimInstance->GetLinkedInputPoseNode())
		{
			if (InOutPose.IsValid())
			{
				InputNode->CachedInputPose.CopyBonesFrom(InOutPose);
				InputNode->CachedInputCurve.CopyFrom(OutCurve);
				InputNode->CachedAttributes.CopyFrom(OutAttributes);
				InputNode->bIsCachedInputPoseInitialized = true;
			}
			else
			{
				const FBoneContainer& RequiredBone = PostProcessAnimInstance->GetRequiredBonesOnAnyThread();
				InputNode->CachedInputPose.ResetToRefPose(RequiredBone);
				InputNode->CachedInputCurve.InitFrom(RequiredBone);
				InputNode->bIsCachedInputPoseInitialized = true;
			}
		}

		EvaluateAnimation(InSkeletalMesh, PostProcessAnimInstance, bInForceRefPose, OutRootBoneTranslation, OutCurve, InOutPose, OutAttributes);
	}
}

// UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
const IClothingSimulation* USkeletalMeshComponent::GetClothingSimulation() const
{
	if (const IClothingSimulationInterface* const ClothingSimulation = GetClothingSimulationInstances().Num() ? ClothingSimulationInstances[0].GetClothingSimulation() : nullptr)
	{
		return ClothingSimulation->DynamicCastToIClothingSimulation();
	}
	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
IClothingSimulation* USkeletalMeshComponent::GetClothingSimulation()
{
	if (IClothingSimulationInterface* const ClothingSimulation = GetClothingSimulationInstances().Num() ? ClothingSimulationInstances[0].GetClothingSimulation() : nullptr)
	{
		return ClothingSimulation->DynamicCastToIClothingSimulation();
	}
	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
const IClothingSimulationContext* USkeletalMeshComponent::GetClothingSimulationContext() const
{
	return GetClothingSimulationInstances().Num() ? ClothingSimulationInstances[0].GetClothingSimulationContext() : nullptr;
}

// UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
IClothingSimulationContext* USkeletalMeshComponent::GetClothingSimulationContext()
{
	check(IsInGameThread());
	return GetClothingSimulationInstances().Num() ? ClothingSimulationInstances[0].GetClothingSimulationContext() : nullptr;
}

UClothingSimulationInteractor* USkeletalMeshComponent::GetClothingSimulationInteractor(const FName ClothingDataName) const
{
	check(IsInGameThread());
	if (ClothingDataName != NAME_None)
	{
		for (const FClothingSimulationInstance& ClothingSimulationInstance : GetClothingSimulationInstances())
		{
			if (UClothingSimulationInteractor* const ClothingSimulationInteractor = ClothingSimulationInstance.GetClothingSimulationInteractor())
			{
				if (ClothingSimulationInteractor->ClothingInteractors.Contains(ClothingDataName))
				{
					return ClothingSimulationInteractor;
				}
			}
		}
	}
	else if (ClothingSimulationInstances.Num())
	{
		// For legacy blueprint compatibility only, return the first simulation's interactor
		UE_CLOGF(
			ClothingSimulationInstances.Num() > 1,
			LogSkeletalMesh,
			Warning,
			"GetClothingSimulationInteractor must be passed a valid Clothing Data name to retrieve a Clothing Simulation Interactor for Skeletal Mesh [%ls], because this asset requires multiple simulation types.",
			GetSkeletalMeshAsset() ? *GetSkeletalMeshAsset()->GetName() : TEXT(""));
		return ClothingSimulationInstances[0].GetClothingSimulationInteractor();
	}
	return nullptr;
}

void USkeletalMeshComponent::CompleteParallelClothSimulation()
{
	if(IsValidRef(ParallelClothTask))
	{
		// No longer need this task, it has completed
		ParallelClothTask.SafeRelease();

		// Write back to the GT cache
		WritebackClothingSimulationData();
	}
}

void USkeletalMeshComponent::UpdateClothSimulationContext(float InDeltaTime)
{
	//Do the teleport cloth test here on the game thread
	CheckClothTeleport();

	bool bMustUpdateClothTransform = bForceCollisionUpdate;

	const bool bNeedsHardReset = ClothTeleportMode == EClothingTeleportMode::HardReset;
	if (bPendingClothTransformUpdate)	//it's possible we want to update cloth collision based on a pending transform
	{
		bPendingClothTransformUpdate = false;
		if (PendingTeleportType == ETeleportType::TeleportPhysics)	//If the pending transform came from a teleport, make sure to teleport the cloth in this upcoming simulation
		{
			ClothTeleportMode = (ClothTeleportMode > EClothingTeleportMode::Teleport) ? ClothTeleportMode : EClothingTeleportMode::Teleport;
		}
		else if (PendingTeleportType == ETeleportType::ResetPhysics)
		{
			ClothTeleportMode = (ClothTeleportMode > EClothingTeleportMode::TeleportAndReset) ? ClothTeleportMode : EClothingTeleportMode::TeleportAndReset;
		}
		bMustUpdateClothTransform = true;
	}
	if (bMustUpdateClothTransform)
	{
		UpdateClothTransformImp();
	}

	// Fill the contexts for the next simulation
	const int32 PredictedLOD = GetPredictedLODLevel();
	for (FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
	{
		constexpr bool bIsInitialization = false;
		ClothingSimulationInstance.FillContextAndPrepareTick(this, InDeltaTime, bIsInitialization);
		if (bNeedsHardReset)
		{
			ClothingSimulationInstance.HardResetSimulation();
		}

		ClothingSimulationInstance.SyncClothingInteractor();
	}

	PendingTeleportType = ETeleportType::None;
	ClothTeleportMode = EClothingTeleportMode::None;
}

void USkeletalMeshComponent::HandleExistingParallelClothSimulation()
{
	if (bBindClothToLeaderComponent)
	{
		if (USkeletalMeshComponent* LeaderComp = Cast<USkeletalMeshComponent>(LeaderPoseComponent.Get()))
		{
			LeaderComp->HandleExistingParallelClothSimulation();
		}
	}

	if(IsValidRef(ParallelClothTask))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EndParallelClothTask);
		CSV_SCOPED_SET_WAIT_STAT(Cloth);

		// There's a simulation in flight
		check(IsInGameThread());
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(ParallelClothTask, ENamedThreads::GameThread);
		CompleteParallelClothSimulation();
	}
}

void USkeletalMeshComponent::WritebackClothingSimulationData()
{
	if (ClothingSimulationInstances.Num())
	{
		CSV_SCOPED_TIMING_STAT(Animation, Cloth);

		const USkinnedMeshComponent* OverrideComponent = nullptr;
		if (LeaderPoseComponent.IsValid())
		{
			OverrideComponent = LeaderPoseComponent.Get();

			// Check if our bone map is actually valid, if not there is no clothing data to build
			if (GetLeaderBoneMap().Num() == 0)
			{
				CurrentSimulationData.Reset();
				return;
			}
		}

		// Reset all existing CurrentSimulationData so that the ClothingSimulationInstances can append their data.
		for (TMap<int32, FClothSimulData>::TIterator DataIter = CurrentSimulationData.CreateIterator(); DataIter; ++DataIter)
		{
			DataIter.Value().Reset();
		}

		for (const FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
		{
			ClothingSimulationInstance.AppendSimulationData(CurrentSimulationData, this, OverrideComponent);
		}

		for (TMap<int32, FClothSimulData>::TIterator DataIter = CurrentSimulationData.CreateIterator(); DataIter; ++DataIter)
		{
			if (DataIter.Value().IsReset())
			{
				DataIter.RemoveCurrent();
			}
		}
	}
}

void USkeletalMeshComponent::DoInstancePreEvaluation()
{
	if (AnimScriptInstance)
	{
		AnimScriptInstance->PreEvaluateAnimation();

		for (UAnimInstance* LinkedInstance : LinkedInstances)
		{
			LinkedInstance->PreEvaluateAnimation();
		}
	}

	if (ShouldEvaluatePostProcessInstance())
	{
		PostProcessAnimInstance->PreEvaluateAnimation();
	}
}

void USkeletalMeshComponent::DoInstancePostEvaluation()
{
	if (AnimScriptInstance)
	{
		AnimScriptInstance->PostEvaluateAnimation();

		for (UAnimInstance* LinkedInstance : LinkedInstances)
		{
			LinkedInstance->PostEvaluateAnimation();
		}
	}

	if (PostProcessAnimInstance)
	{
		PostProcessAnimInstance->PostEvaluateAnimation();
	}
}

void USkeletalMeshComponent::DoInstanceFinalizeAnimation(bool bWasEvaluated)
{
	if (AnimScriptInstance)
	{
		AnimScriptInstance->FinalizeAnimation(bWasEvaluated);

		for (UAnimInstance* LinkedInstance : LinkedInstances)
		{
			LinkedInstance->FinalizeAnimation(bWasEvaluated);
		}
	}

	if (PostProcessAnimInstance)
	{
		PostProcessAnimInstance->FinalizeAnimation(bWasEvaluated);
	}
}


void USkeletalMeshComponent::RefreshBoneTransforms(FActorComponentTickFunction* TickFunction)
{
	if(!bEnableAnimation)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_AnimGameThreadTime);
	SCOPE_CYCLE_COUNTER(STAT_RefreshBoneTransforms);

	check(IsInGameThread()); //Only want to call this from the game thread as we set up tasks etc
	
	if (!GetSkeletalMeshAsset() || GetNumComponentSpaceTransforms() == 0)
	{
		return;
	}

	// Recalculate the RequiredBones array, if necessary
	if (!bRequiredBonesUpToDate)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_USkeletalMeshComponent_RefreshBoneTransforms_RecalcRequiredBones);
		RecalcRequiredBones(GetPredictedLODLevel());
	}
	// if curves have to be refreshed
	else if (!AreRequiredCurvesUpToDate())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_USkeletalMeshComponent_RefreshBoneTransforms_RecalcRequiredCurves);
		RecalcRequiredCurves();
	}

	const bool bCachedShouldUseUpdateRateOptimizations = ShouldUseUpdateRateOptimizations() && AnimUpdateRateParams != nullptr;
	const bool bDoEvaluationRateOptimization = (bExternalTickRateControlled && bExternalEvaluationRateLimited) || (bCachedShouldUseUpdateRateOptimizations && AnimUpdateRateParams->DoEvaluationRateOptimizations());

	//Handle update rate optimization setup
	//Dont mark cache as invalid if we aren't performing optimization anyway
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bInvalidCachedBones = bDoEvaluationRateOptimization &&
									 ((BoneSpaceTransforms.Num() != GetSkeletalMeshAsset()->GetRefSkeleton().GetNum())
									 || (BoneSpaceTransforms.Num() != CachedBoneSpaceTransforms.Num())
									 || (GetNumComponentSpaceTransforms() != CachedComponentSpaceTransforms.Num()));
	
	const bool bInvalidCachedCurve = bDoEvaluationRateOptimization && CachedCurve.Num() != AnimCurves.Num();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	const bool bInvalidCachedAttributes = bDoEvaluationRateOptimization && CachedAttributes != CustomAttributes;

	const bool bShouldDoEvaluation = !bDoEvaluationRateOptimization || bInvalidCachedBones || bInvalidCachedCurve || (bExternalTickRateControlled && bExternalUpdate) || (bCachedShouldUseUpdateRateOptimizations && !AnimUpdateRateParams->ShouldSkipEvaluation());

	const bool bShouldInterpolateSkippedFrames = (bExternalTickRateControlled && bExternalInterpolate) || (bCachedShouldUseUpdateRateOptimizations && AnimUpdateRateParams->ShouldInterpolateSkippedFrames());

	const bool bShouldDoInterpolation = TickFunction != nullptr && bDoEvaluationRateOptimization && !bInvalidCachedBones && bShouldInterpolateSkippedFrames;

	const bool bShouldDoParallelInterpolation = bShouldDoInterpolation && CVarUseParallelAnimationInterpolation.GetValueOnGameThread() == 1;

	const bool bDoPAE = !!CVarUseParallelAnimationEvaluation.GetValueOnGameThread() && (FApp::ShouldUseThreadingForPerformance() || FForkProcessHelper::SupportsMultithreadingPostFork());

	const bool bMainInstanceValidForParallelWork = AnimScriptInstance == nullptr || AnimScriptInstance->CanRunParallelWork();
	const bool bPostInstanceValidForParallelWork = PostProcessAnimInstance == nullptr || PostProcessAnimInstance->CanRunParallelWork();
	const bool bHasValidInstanceForParallelWork = HasValidAnimationInstance() && bMainInstanceValidForParallelWork && bPostInstanceValidForParallelWork;
	const bool bDoParallelEvaluation = bHasValidInstanceForParallelWork && bDoPAE && (bShouldDoEvaluation || bShouldDoParallelInterpolation) && TickFunction && TickFunction->IsCompletionHandleValid();
	const bool bBlockOnTask = !bDoParallelEvaluation;  // If we aren't trying to do parallel evaluation then we
															// will need to wait on an existing task.

	const bool bPerformPostAnimEvaluation = true;
	if (HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation))
	{
		return;
	}

	AnimEvaluationContext.SkeletalMesh = GetSkeletalMeshAsset();
	AnimEvaluationContext.AnimInstance = AnimScriptInstance;
	AnimEvaluationContext.PostProcessAnimInstance = (ShouldEvaluatePostProcessInstance())? ToRawPtr(PostProcessAnimInstance): nullptr;

	AnimEvaluationContext.bDoEvaluation = bShouldDoEvaluation;
	AnimEvaluationContext.bDoInterpolation = bShouldDoInterpolation;
	AnimEvaluationContext.bDuplicateToCacheBones = bInvalidCachedBones || (bDoEvaluationRateOptimization && AnimEvaluationContext.bDoEvaluation && !AnimEvaluationContext.bDoInterpolation);
	AnimEvaluationContext.bDuplicateToCacheCurve = bInvalidCachedCurve || (bDoEvaluationRateOptimization && AnimEvaluationContext.bDoEvaluation && !AnimEvaluationContext.bDoInterpolation);

	AnimEvaluationContext.bDuplicateToCachedAttributes = bInvalidCachedAttributes || (bDoEvaluationRateOptimization && AnimEvaluationContext.bDoEvaluation && !AnimEvaluationContext.bDoInterpolation);
	AnimEvaluationContext.bForceRefPose = bForceRefpose;

	if (!bDoEvaluationRateOptimization)
	{
		//If we aren't optimizing clear the cached local atoms
		CachedBoneSpaceTransforms.Reset();
		CachedComponentSpaceTransforms.Reset();
		CachedCurve.Empty();
		CachedAttributes.Empty();
	}

	if (bShouldDoEvaluation)
	{
		// If we're going to evaluate animation, call PreEvaluateAnimation()
		{
			DoInstancePreEvaluation();
		}

		// If we need to eval the graph, and we're not going to update it.
		// make sure it's been ticked at least once!
		{
			bool bShouldTickAnimation = false;		
			if (AnimScriptInstance && !AnimScriptInstance->NeedsUpdate())
			{
				bShouldTickAnimation = !AnimScriptInstance->GetUpdateCounter().HasEverBeenUpdated();
			}

			if (!bDisablePostProcessBlueprint && PostProcessAnimInstance && ShouldEvaluatePostProcessAnimBP() && !PostProcessAnimInstance->NeedsUpdate())
			{
				bShouldTickAnimation = bShouldTickAnimation || !PostProcessAnimInstance->GetUpdateCounter().HasEverBeenUpdated();
			}

			if (bShouldTickAnimation)
			{
				// We bypass TickPose() and call TickAnimation directly, so URO doesn't intercept us.
				TickAnimation(0.f, false);
			}
		}
	}

	if (bDoParallelEvaluation)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_USkeletalMeshComponent_RefreshBoneTransforms_SetupParallel);

		DispatchParallelEvaluationTasks(TickFunction);
	}
	else
	{
		if (AnimEvaluationContext.bDoEvaluation || AnimEvaluationContext.bDoInterpolation)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_USkeletalMeshComponent_RefreshBoneTransforms_GamethreadEval);

			DoParallelEvaluationTasks_OnGameThread();
		}
		else
		{
			if (!AnimEvaluationContext.bDoInterpolation)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_USkeletalMeshComponent_RefreshBoneTransforms_CopyBones);

				if(CachedBoneSpaceTransforms.Num())
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					BoneSpaceTransforms.Reset();
					BoneSpaceTransforms.Append(CachedBoneSpaceTransforms);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				if(CachedComponentSpaceTransforms.Num())
				{
					TArray<FTransform>& LocalEditableSpaceBases = GetEditableComponentSpaceTransforms();
					LocalEditableSpaceBases.Reset();
					LocalEditableSpaceBases.Append(CachedComponentSpaceTransforms);
				}
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				AnimCurves.CopyFrom(CachedCurve);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				if (CachedAttributes.ContainsData())
				{
					CustomAttributes.CopyFrom(CachedAttributes);
				}
			}
			if(AnimScriptInstance && AnimScriptInstance->NeedsUpdate())
			{
				AnimScriptInstance->ParallelUpdateAnimation();
			}

			if(ShouldPostUpdatePostProcessInstance())
			{
				PostProcessAnimInstance->ParallelUpdateAnimation();
			}
		}

		PostAnimEvaluation(AnimEvaluationContext);
		AnimEvaluationContext.Clear();
	}

	if (TickFunction == nullptr && ShouldBlendPhysicsBones())
	{
		//Since we aren't doing this through the tick system, and we wont have done it in PostAnimEvaluation, assume we want the buffer flipped now
		FinalizeBoneTransform();
	}
}

void USkeletalMeshComponent::SwapEvaluationContextBuffers()
{
	Exchange(AnimEvaluationContext.ComponentSpaceTransforms, GetEditableComponentSpaceTransforms());
	Exchange(AnimEvaluationContext.CachedComponentSpaceTransforms, CachedComponentSpaceTransforms);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Exchange(AnimEvaluationContext.BoneSpaceTransforms, BoneSpaceTransforms);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Exchange(AnimEvaluationContext.CachedBoneSpaceTransforms, CachedBoneSpaceTransforms);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Exchange(AnimEvaluationContext.Curve, AnimCurves);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Exchange(AnimEvaluationContext.CachedCurve, CachedCurve);
	Exchange(AnimEvaluationContext.RootBoneTranslation, RootBoneTranslation);

	Exchange(AnimEvaluationContext.CustomAttributes, CustomAttributes);
	Exchange(AnimEvaluationContext.CachedCustomAttributes, CachedAttributes);
}

void USkeletalMeshComponent::DispatchParallelEvaluationTasks(FActorComponentTickFunction* TickFunction)
{
	// If we try and start parallel evaluation in a transaction, just run it immediately instead as it
	// is unsafe to have a transactional thread interacting with anything async.
	if (AutoRTFM::IsTransactional())
	{
		ensureMsgf(false, TEXT("Should not be calling `DispatchParallelEvaluationTasks` from within a transaction!"));
		check(!IsValidRef(ParallelAnimationEvaluationTask));
		DoParallelEvaluationTasks_OnGameThread();
		return;
	}

	LLM_SCOPE(ELLMTag::SkeletalMesh);
	SwapEvaluationContextBuffers();

#if WITH_EDITOR
	// We can only finish compilation on the game-thread, so wait here before spawning eval tasks.
	if (GetSkeletalMeshAsset() && GetSkeletalMeshAsset()->IsCompiling())
	{
		FSkinnedAssetCompilingManager::Get().FinishCompilation({ GetSkeletalMeshAsset() });
	}
#endif

	// start parallel work
	check(!IsValidRef(ParallelAnimationEvaluationTask));
	ParallelAnimationEvaluationTask = TGraphTask<FParallelAnimationEvaluationTask>::CreateTask().ConstructAndDispatchWhenReady(this);

	// set up a task to run on the game thread to accept the results
	FGraphEventArray Prerequistes;
	Prerequistes.Add(ParallelAnimationEvaluationTask);
	FGraphEventRef TickCompletionEvent = TGraphTask<FParallelAnimationCompletionTask>::CreateTask(&Prerequistes).ConstructAndDispatchWhenReady(this);

	if ( TickFunction )
	{
		TickFunction->GetCompletionHandle()->DontCompleteUntil(TickCompletionEvent);
	}
}

void USkeletalMeshComponent::DoParallelEvaluationTasks_OnGameThread()
{
#if WITH_EDITOR
	// If we are re-instancing, we will not have run an update before since 
	// bNeedsUpdate == false and UAnimInstance::UpdateAnimation gets skipped
	if (GIsReinstancing)
	{
		return;
	}
#endif


	SwapEvaluationContextBuffers();

	ParallelAnimationEvaluation();

	SwapEvaluationContextBuffers();
}

void USkeletalMeshComponent::DispatchParallelTickPose(FActorComponentTickFunction* TickFunction)
{
	check(TickFunction);
	
	if(GetSkeletalMeshAsset() != nullptr)
	{
		if ((AnimScriptInstance && AnimScriptInstance->NeedsUpdate()) ||
			(PostProcessAnimInstance && PostProcessAnimInstance->NeedsUpdate()))
		{
			if (ShouldTickAnimation())
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_USkeletalMeshComponent_RefreshBoneTransforms_DispatchParallelTickPose);

				// This duplicates *some* of the logic from RefreshBoneTransforms()
				const bool bDoPAE = !!CVarUseParallelAnimationEvaluation.GetValueOnGameThread() && (FApp::ShouldUseThreadingForPerformance() || FForkProcessHelper::SupportsMultithreadingPostFork());

				const bool bDoParallelUpdate = bDoPAE && TickFunction->IsCompletionHandleValid();

				const bool bBlockOnTask = !bDoParallelUpdate;   // If we aren't trying to do parallel update then we
																// will need to wait on an existing task.

				const bool bPerformPostAnimEvaluation = true;
				if (HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation))
				{
					return;
				}

				// Do a mini-setup of the eval context
				AnimEvaluationContext.SkeletalMesh = GetSkeletalMeshAsset();
				AnimEvaluationContext.AnimInstance = AnimScriptInstance;

				// We dont set up the Curve here, as we dont use it in Update()
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				AnimCurves.Empty();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				CustomAttributes.Empty();

				// Set us up to NOT perform evaluation
				AnimEvaluationContext.bDoEvaluation = false;
				AnimEvaluationContext.bDoInterpolation = false;
				AnimEvaluationContext.bDuplicateToCacheBones = false;
				AnimEvaluationContext.bDuplicateToCacheCurve = false;
				AnimEvaluationContext.bDuplicateToCachedAttributes = false;

				// Ensure that internal objects are cached before we run any anim graph logic
				DoInstancePreEvaluation();

				if(bDoParallelUpdate)
				{
					DispatchParallelEvaluationTasks(TickFunction);
				}
				else
				{
					// we cant update on a worker thread, so perform the work here
					DoParallelEvaluationTasks_OnGameThread();
					PostAnimEvaluation(AnimEvaluationContext);
				}
			}
		}
	}
}

void USkeletalMeshComponent::PostAnimEvaluation(FAnimationEvaluationContext& EvaluationContext)
{
#if DO_CHECK
	checkf(!bPostEvaluatingAnimation, TEXT("PostAnimEvaluation already in progress, recursion detected for SkeletalMeshComponent [%s], AnimInstance [%s]"), *GetPathNameSafe(this), *GetPathNameSafe(EvaluationContext.AnimInstance));

	FGuardValue_Bitfield(bPostEvaluatingAnimation, true);
#endif

	SCOPE_CYCLE_COUNTER(STAT_PostAnimEvaluation);

	if (EvaluationContext.AnimInstance)
	{
		EvaluationContext.AnimInstance->PostUpdateAnimation();
	}

	if (ShouldPostUpdatePostProcessInstance())
	{
		PostProcessAnimInstance->PostUpdateAnimation();
	}

	if (!IsRegistered() && !IsPreRegistering()) // Notify/Event has caused us to go away so cannot carry on from here
	{
		return;
	}

	if(CVarUseParallelAnimationInterpolation.GetValueOnGameThread() == 0)
	{
		if (EvaluationContext.bDuplicateToCacheCurve)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CachedCurve.CopyFrom(AnimCurves);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
		if (EvaluationContext.bDuplicateToCachedAttributes)
		{
			CachedAttributes.CopyFrom(CustomAttributes);
		}
	
		if (EvaluationContext.bDuplicateToCacheBones)
		{
			CachedComponentSpaceTransforms.Reset();
			CachedComponentSpaceTransforms.Append(GetEditableComponentSpaceTransforms());
			CachedBoneSpaceTransforms.Reset();
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CachedBoneSpaceTransforms.Append(BoneSpaceTransforms);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		if (EvaluationContext.bDoInterpolation)
		{
			SCOPE_CYCLE_COUNTER(STAT_InterpolateSkippedFrames);

			float Alpha;
			if(bEnableUpdateRateOptimizations && AnimUpdateRateParams)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (AnimScriptInstance)
				{
					AnimScriptInstance->OnUROPreInterpolation();
				}

				for(UAnimInstance* LinkedInstance : LinkedInstances)
				{
					LinkedInstance->OnUROPreInterpolation();
				}

				if(PostProcessAnimInstance)
				{
					PostProcessAnimInstance->OnUROPreInterpolation();
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				Alpha = AnimUpdateRateParams->GetInterpolationAlpha();
			}
			else
			{
				Alpha = ExternalInterpolationAlpha;
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FAnimationRuntime::LerpBoneTransforms(BoneSpaceTransforms, CachedBoneSpaceTransforms, Alpha, RequiredBones);
			GetSkeletalMeshAsset()->FillComponentSpaceTransforms(BoneSpaceTransforms, FillComponentSpaceTransformsRequiredBones, GetEditableComponentSpaceTransforms());

			// interpolate curve
			AnimCurves.LerpTo(CachedCurve, Alpha);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			// Interpolate custom attributes
			UE::Anim::Attributes::InterpolateAttributes(CustomAttributes, CachedAttributes, Alpha);
		}
	}

	// Work below only matters if bone transforms have been updated.
	// i.e. if we're using URO and skipping a frame with no interpolation, 
	// we don't need to do that work.
	if (EvaluationContext.bDoEvaluation || EvaluationContext.bDoInterpolation)
	{
		// clear morphtarget curve sets since we're going to apply new changes
		ResetMorphTargetCurves();

		if(AnimScriptInstance)
		{
#if WITH_EDITOR
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			GetEditableAnimationCurves() = AnimCurves;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif 
			GetEditableCustomAttributes() = CustomAttributes;

			// curve update happens first
			AnimScriptInstance->UpdateCurvesPostEvaluation();

			// this is same curves, and we don't have to process same for everything. 
			// we just copy curves from main for the case where GetCurveValue works in that instance
			for(UAnimInstance* LinkedInstance : LinkedInstances)
			{
				LinkedInstance->CopyCurveValues(*AnimScriptInstance);
			}
		}

		// now update morphtarget curves that are added via SetMorphTarget
		UpdateMorphTargetOverrideCurves();

		if(PostProcessAnimInstance)
		{
			if (AnimScriptInstance)
			{
				// this is same curves, and we don't have to process same for everything. 
				// we just copy curves from main for the case where GetCurveValue works in that instance
				PostProcessAnimInstance->CopyCurveValues(*AnimScriptInstance);
			}
			else
			{
				// if no main anim instance, we'll have to have post processor to handle it
				PostProcessAnimInstance->UpdateCurvesPostEvaluation();
			}
		}

		// If we have actually evaluated animations, we need to call PostEvaluateAnimation now.
		if (EvaluationContext.bDoEvaluation)
		{
			DoInstancePostEvaluation();
		}

		DoInstanceFinalizeAnimation(EvaluationContext.bDoEvaluation);

		bNeedToFlipSpaceBaseBuffers = true;

		if (Bodies.Num() > 0 || bEnablePerPolyCollision)
		{
			// update physics data from animated data
			if(bSkipKinematicUpdateWhenInterpolating)
			{
				if(EvaluationContext.bDoEvaluation)
				{
					// push newly evaluated bones to physics
					UpdateKinematicBonesToAnim(EvaluationContext.bDoInterpolation ? CachedComponentSpaceTransforms : GetEditableComponentSpaceTransforms(), ETeleportType::None, true);
					UpdateRBJointMotors();
				}
			}
			else
			{
				UpdateKinematicBonesToAnim(GetEditableComponentSpaceTransforms(), ETeleportType::None, true);
				UpdateRBJointMotors();
			}
		}


#if WITH_EDITOR	
		// If we have no physics to blend or in editor since there is no physics tick group, we are done
		if (!ShouldBlendPhysicsBones() || GetWorld()->WorldType == EWorldType::Editor)
		{
			// Flip buffers, update bounds, attachments etc.
			FinalizeAnimationUpdate();
		}
#else
		if (!ShouldBlendPhysicsBones())
		{
			// Flip buffers, update bounds, attachments etc.
			FinalizeAnimationUpdate();
		}
#endif
	}
	else 
	{
		DoInstanceFinalizeAnimation(false);

		// Since we're not calling FinalizeBoneTransforms via PostBlendPhysics,
		// make sure we call ConditionallyDispatchQueuedAnimEvents() in case we ticked, but didn't evalutate.

		/////////////////////////////////////////////////////////////////////////////
		// Notify / Event Handling!
		// This can do anything to our component (including destroy it) 
		// Any code added after this point needs to take that into account
		/////////////////////////////////////////////////////////////////////////////

		ConditionallyDispatchQueuedAnimEvents();
	}

	AnimEvaluationContext.Clear();
}

void USkeletalMeshComponent::ApplyAnimationCurvesToComponent(const TMap<FName, float>* InMaterialParameterCurves, const TMap<FName, float>* InAnimationMorphCurves)
{
	const bool bContainsMaterialCurves = InMaterialParameterCurves && InMaterialParameterCurves->Num() > 0;
	if (bContainsMaterialCurves)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FAnimInstanceProxy_UpdateComponentsMaterialParameters);
		for (auto Iter = InMaterialParameterCurves->CreateConstIterator(); Iter; ++Iter)
		{
			FName ParameterName = Iter.Key();
			float ParameterValue = Iter.Value();
			SetScalarParameterValueOnMaterials(ParameterName, ParameterValue);
		}
	}

	const bool bContainsMorphCurves = InAnimationMorphCurves && InAnimationMorphCurves->Num() > 0;
	if (GetSkeletalMeshAsset() && bContainsMorphCurves)
	{
		// we want to append to existing curves - i.e. BP driven curves 
		FAnimationRuntime::AppendActiveMorphTargets(GetSkeletalMeshAsset(), *InAnimationMorphCurves, ActiveMorphTargets, MorphTargetWeights);
	}

	/** Push through curves to follower components */
	if (bPropagateCurvesToFollowers && (bContainsMorphCurves || bContainsMaterialCurves) && FollowerPoseComponents.Num() > 0)
	{
		for (TWeakObjectPtr<USkinnedMeshComponent> MeshComponent : FollowerPoseComponents)
		{
			if (USkeletalMeshComponent* SKComponent = Cast<USkeletalMeshComponent>(MeshComponent.Get()))
			{
				SKComponent->ApplyAnimationCurvesToComponent(InMaterialParameterCurves, InAnimationMorphCurves);
			}
		}
	}
}

FBoxSphereBounds USkeletalMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	SCOPE_CYCLE_COUNTER(STAT_CalcSkelMeshBounds);

	// fixme laurent - extend concept of LocalBounds to all SceneComponent
	// as rendered calls CalcBounds*() directly in FScene::UpdatePrimitiveTransform, which is pretty expensive for SkelMeshes.
	// No need to calculated that again, just use cached local bounds.
	if (bCachedWorldSpaceBoundsUpToDate || bCachedLocalBoundsUpToDate)
	{
		FBoxSphereBounds Result;
		if (bCachedLocalBoundsUpToDate)
		{
			Result = CachedWorldOrLocalSpaceBounds.TransformBy(LocalToWorld);
		}
		else
		{
			Result = CachedWorldOrLocalSpaceBounds.TransformBy(CachedWorldToLocalTransform * LocalToWorld.ToMatrixWithScale());
		}

		if (bIncludeComponentLocationIntoBounds)
		{
			const FVector ComponentLocation = GetComponentLocation();
			return Result + FBoxSphereBounds(ComponentLocation, FVector(1.0f), 1.0f);
		}
		else
		{
			return Result;
		}
	}
	// Calculate new bounds
	else
	{
		FVector RootBoneOffset = RootBoneTranslation;

		// if to use LeaderPoseComponent's fixed skel bounds, 
		// send LeaderPoseComponent's Root Bone Translation
		if (LeaderPoseComponent.IsValid())
		{
			const USkinnedMeshComponent* const LeaderPoseComponentInst = LeaderPoseComponent.Get();
			check(LeaderPoseComponentInst);
			if (LeaderPoseComponentInst->GetSkinnedAsset() &&
				LeaderPoseComponentInst->bComponentUseFixedSkelBounds &&
				LeaderPoseComponentInst->IsA((USkeletalMeshComponent::StaticClass())))
			{
				const USkeletalMeshComponent* BaseComponent = CastChecked<USkeletalMeshComponent>(LeaderPoseComponentInst);
				RootBoneOffset = BaseComponent->RootBoneTranslation; // Adjust bounds by root bone translation
			}
		}

		const bool bCacheLocalSpaceBounds = CVarCacheLocalSpaceBounds.GetValueOnAnyThread() != 0;
		
		const FTransform CachedBoundsTransform = bCacheLocalSpaceBounds ? FTransform::Identity : LocalToWorld;

		FBoxSphereBounds NewBounds = CalcMeshBound((FVector3f)RootBoneOffset, bHasValidBodies, CachedBoundsTransform);

		if (bIncludeComponentLocationIntoBounds)
		{
			const FVector ComponentLocation = GetComponentLocation();
			const FBoxSphereBounds ComponentLocationBounds(ComponentLocation, FVector(1.0f), 1.0f);
			if (bCacheLocalSpaceBounds)
			{
				NewBounds = NewBounds.TransformBy(LocalToWorld);
				NewBounds = NewBounds + ComponentLocationBounds;
				NewBounds = NewBounds.TransformBy(LocalToWorld.ToInverseMatrixWithScale());
			}
			else
			{
				NewBounds = NewBounds + ComponentLocationBounds;
			}			
		}

		AddClothingBounds(NewBounds, CachedBoundsTransform);

		CachedWorldOrLocalSpaceBounds = NewBounds;
		bCachedLocalBoundsUpToDate = bCacheLocalSpaceBounds;
		bCachedWorldSpaceBoundsUpToDate = !bCacheLocalSpaceBounds;

		if (bCacheLocalSpaceBounds)
		{ 
			CachedWorldToLocalTransform.SetIdentity();
			return NewBounds.TransformBy(LocalToWorld);
		}
		else
		{
			CachedWorldToLocalTransform = LocalToWorld.ToInverseMatrixWithScale();
			return NewBounds;
		}

	}
}

void USkeletalMeshComponent::SetSkeletalMesh(USkeletalMesh* InSkelMesh, bool bReinitPose)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SetSkeletalMesh);
	SCOPE_CYCLE_UOBJECT(NewSkelMesh, InSkelMesh);

	if (InSkelMesh == GetSkeletalMeshAsset())
	{
		// do nothing if the input mesh is the same mesh we're already using.
		return;
	}

	// Stop any existing cloth simulation prior to replacing the asset
	ReleaseAllClothingResources();

	// Update property alias
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMeshAsset = InSkelMesh;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// We may be doing parallel evaluation on the current anim instance
	// Calling this here with true will block this init till that thread completes
	// and it is safe to continue
	const bool bBlockOnTask = true; // wait on evaluation task so it is safe to continue with Init
	const bool bPerformPostAnimEvaluation = true;
	HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation);

	UPhysicsAsset* OldPhysAsset = GetPhysicsAsset();

	{
		FRenderStateRecreator RenderStateRecreator(this);

		// The SetSkeletalMesh base implementation is being phased out of USkinnedMeshComponent, and SetSkinnedAssetAndUpdate replaces it,
		// but the SetSkeletalMesh function will remain to provide continuity and a more convenient USkeletalMesh based API avoiding the uncertainty of the Cast to USkeletalMesh.
		// Also, since USkeletalMeshComponent::SetSkinnedAssetAndUpdate now calls USkeletalMeshComponent::SetSkeletalMesh, and since SetSkinnedAssetAndUpdate is virtual,
		// the Super keyword is necessary to make sure that the correct USkinnedMeshComponent base function is called and prevent a recursive call loop.
		Super::SetSkinnedAssetAndUpdate(InSkelMesh, bReinitPose);
		
#if WITH_EDITOR
		ValidateAnimation();
#endif

		if(IsPhysicsStateCreated())
		{
			// Make sure that required bones are invalidated as we have just changed our mesh
			// RecalcRequiredBones will be called by RecreatePhysicsState (which may be called by UpdateBoneBodyMapping).
			bRequiredBonesUpToDate = false;

			if(GetPhysicsAsset() == OldPhysAsset && OldPhysAsset && Bodies.Num() == OldPhysAsset->SkeletalBodySetups.Num())	//Make sure that we actually created all the bodies for the asset (needed for old assets in editor)
			{
				UpdateBoneBodyMapping();
			}
			else
			{
				RecreatePhysicsState();
			}
		}

		UpdateHasValidBodies();
		ClearMorphTargets();

		// Make sure that required bones are invalidated as we have just changed our mesh
		// RecalcRequiredBones will be called by InitAnim below
		bRequiredBonesUpToDate = false;
		
		if(bEnableAnimation)
		{
			InitAnim(bReinitPose);
		}
		else if(IsRegistered() || IsPreRegistering())
		{
			RecalcRequiredBones(GetPredictedLODLevel());
			ResetToRefPose();
			bBoneTransformsInitialized = true;
		}

		RecreateClothingActors();
	}

	// Mark cached material parameter names dirty
	MarkCachedMaterialParameterNameIndicesDirty();

	// Update this component streaming data.
	IStreamingManager::Get().NotifyPrimitiveUpdated(this);
}

void USkeletalMeshComponent::SetSkinnedAssetAndUpdate(USkinnedAsset* InSkinnedAsset, bool bReinitPose)
{
	SetSkeletalMesh(Cast<USkeletalMesh>(InSkinnedAsset), bReinitPose);
}

void USkeletalMeshComponent::SetSkeletalMeshWithoutResettingAnimation(USkeletalMesh* InSkelMesh)
{
	SetSkeletalMesh(InSkelMesh,false);
}

bool USkeletalMeshComponent::AllocateTransformData()
{
	LLM_SCOPE_BYNAME("SkeletalMesh/TransformData");

	// Allocate transforms if not present.
	if ( Super::AllocateTransformData() )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if(BoneSpaceTransforms.Num() != GetSkeletalMeshAsset()->GetRefSkeleton().GetNum() )
		{
			BoneSpaceTransforms = GetSkeletalMeshAsset()->GetRefSkeleton().GetRefBonePose();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		GetMutableValidMeshPoseTransforms().Init(false, GetNumComponentSpaceTransforms());
		bValidMeshPoseTransformsUpToDate = false;

		return true;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BoneSpaceTransforms.Empty();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	GetMutableValidMeshPoseTransforms().Empty();
	return false;
}

void USkeletalMeshComponent::DeallocateTransformData()
{
	Super::DeallocateTransformData();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BoneSpaceTransforms.Empty();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void USkeletalMeshComponent::SetForceRefPose(bool bNewForceRefPose)
{
	bForceRefpose = bNewForceRefPose;
	MarkRenderStateDirty();
}

void USkeletalMeshComponent::ToggleDisablePostProcessBlueprint()
{
	SetDisablePostProcessBlueprint(!bDisablePostProcessBlueprint);
}

bool USkeletalMeshComponent::GetDisablePostProcessBlueprint() const
{
	return bDisablePostProcessBlueprint;
}

void USkeletalMeshComponent::SetDisablePostProcessBlueprint(bool bInDisablePostProcess)
{
	// If we're re-enabling - reinitialize the post process instance as it may
	// not have been ticked in some time
	if(!bInDisablePostProcess && bDisablePostProcessBlueprint && PostProcessAnimInstance)
	{
		PostProcessAnimInstance->InitializeAnimation();
	}

	bDisablePostProcessBlueprint = bInDisablePostProcess;
}

void USkeletalMeshComponent::K2_SetAnimInstanceClass(class UClass* NewClass)
{
	SetAnimInstanceClass(NewClass);
}

void USkeletalMeshComponent::SetAnimClass(class UClass* NewClass)
{
	SetAnimInstanceClass(NewClass);
}

class UClass* USkeletalMeshComponent::GetAnimClass()
{
	return AnimClass;
}

void USkeletalMeshComponent::SetAnimInstanceClass(class UClass* NewClass)
{
	if (bPostEvaluatingAnimation
#if DO_CHECK
		|| (AnimScriptInstance && (AnimScriptInstance->IsUpdatingAnimation() || AnimScriptInstance->IsPostUpdatingAnimation()))
#endif
		)
	{
		UE_LOGF(LogAnimation, Warning, "USkeletalMeshComponent::SetAnimInstanceClass: Cannot set anim instance class while animation is still updating/evaluating. Apply a delay or perform the switch elsewhere.");
		return;
	}
	if (NewClass != nullptr)
	{
		// set the animation mode
		const bool bWasUsingBlueprintMode = AnimationMode == EAnimationMode::AnimationBlueprint;
		AnimationMode = EAnimationMode::Type::AnimationBlueprint;

		if (NewClass != AnimClass || !bWasUsingBlueprintMode)
		{
			// Only need to initialize if it hasn't already been set or we weren't previously using a blueprint instance
			AnimClass = NewClass;
			ClearAnimScriptInstance();
			InitAnim(true);
		}
	}
	else
	{
		// Need to clear the instance as well as the blueprint.
		// @todo is this it?
		AnimClass = nullptr;
		ClearAnimScriptInstance();
	}
}

void USkeletalMeshComponent::SetEnableAnimation(bool bNewEnableAnimation)
{
	// If we're not registered yet or our parent actor is running construction scripts, we can set the value straight away.
	AActor* OwningActor = GetOwner();
	const bool bRunningConstructionScript = OwningActor && OwningActor->IsRunningUserConstructionScript();
	if (IsRegistered() == false || bRunningConstructionScript)
	{
		bEnableAnimation = bNewEnableAnimation;
		PrimaryComponentTick.bRunOnAnyThread = !bEnableAnimation;
		return;
	}

	// Defer setting flag to the end of the frame as other tasks can be in-flight during a tick
	FTSTicker::GetCoreTicker().AddTicker(ANSI_TO_TCHAR(__FUNCTION__), 0.0f,
		[bNewEnableAnimation, WeakComponent = TWeakObjectPtr<USkeletalMeshComponent>(this)](float InDeltaTime)
		{
			if(USkeletalMeshComponent* Component = WeakComponent.Get())
			{
				if (bNewEnableAnimation != Component->bEnableAnimation)
				{
					Component->bEnableAnimation = bNewEnableAnimation;
					Component->PrimaryComponentTick.bRunOnAnyThread = !bNewEnableAnimation;

					if (bNewEnableAnimation && Component->IsRegistered())
					{
						Component->InitAnim(false);
					}
				}
			}
			return false;
		});
}

UAnimInstance* USkeletalMeshComponent::GetAnimInstance() const
{
	return AnimScriptInstance;
}

void USkeletalMeshComponent::SetAnimInstanceClass_Internal(const TSubclassOf<UAnimInstance>& InAnimClass)
{
	SetAnimInstanceClass(InAnimClass.Get());
}

UAnimInstance* USkeletalMeshComponent::GetPostProcessInstance() const
{
	return PostProcessAnimInstance;
}

void USkeletalMeshComponent::ResetLinkedAnimInstances()
{
	// Reset linked anim layers shared data 
	if (AnimScriptInstance)
	{
		if (FAnimSubsystem_SharedLinkedAnimLayers* SharedLinkedAnimLayers = FAnimSubsystem_SharedLinkedAnimLayers::GetFromMesh(this))
		{
			SharedLinkedAnimLayers->Reset();
		}
	}

	for(UAnimInstance* LinkedInstance : LinkedInstances)
	{
		if(LinkedInstance && LinkedInstance->bCreatedByLinkedAnimGraph)
		{
			LinkedInstance->UninitializeAnimation();
			LinkedInstance->EndNotifyStates();
			LinkedInstance->MarkAsGarbage();
			LinkedInstance = nullptr;
		}
	}
	LinkedInstances.Reset();
}

void USkeletalMeshComponent::ResetToRefPose()
{
	if (USkeletalMesh* SkelMesh = GetSkeletalMeshAsset())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BoneSpaceTransforms = SkelMesh->GetRefSkeleton().GetRefBonePose();
		//Mini RefreshBoneTransforms (the bit we actually care about)
		SkelMesh->FillComponentSpaceTransforms(BoneSpaceTransforms, FillComponentSpaceTransformsRequiredBones, GetEditableComponentSpaceTransforms());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bNeedToFlipSpaceBaseBuffers = true; // Have updated space bases so need to flip
		FlipEditableSpaceBases();
	}
}

void USkeletalMeshComponent::AllowQueuedAnimEventsNextDispatch()
{
	if (!bSuppressNotifyEventDispatch)
	{
		bNeedsQueuedAnimEventsDispatched = true;
	}
	else
	{
		UE_LOGF(LogSkeletalMesh, Warning, "AllowQueuedAnimEventsNextDispatch called when notify dispatch is disabled on mesh %ls. Re-enable dispatch to allow this.", *GetName());
	}	
}

UAnimInstance* USkeletalMeshComponent::GetLinkedAnimGraphInstanceByTag(FName InName) const
{
	if(bEnableAnimation && AnimScriptInstance)
	{
		return AnimScriptInstance->GetLinkedAnimGraphInstanceByTag(InName);
	}
	return nullptr;
}

void USkeletalMeshComponent::GetLinkedAnimGraphInstancesByTag(FName InTag, TArray<UAnimInstance*>& OutLinkedInstances) const
{
	if(bEnableAnimation && AnimScriptInstance)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AnimScriptInstance->GetLinkedAnimGraphInstancesByTag(InTag, OutLinkedInstances);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void USkeletalMeshComponent::LinkAnimGraphByTag(FName InTag, TSubclassOf<UAnimInstance> InClass)
{
	if(bEnableAnimation && AnimScriptInstance)
	{
		AnimScriptInstance->LinkAnimGraphByTag(InTag, InClass);
	}
}

void USkeletalMeshComponent::LinkAnimClassLayers(TSubclassOf<UAnimInstance> InClass)
{
	if(bEnableAnimation && AnimScriptInstance)
	{
		AnimScriptInstance->LinkAnimClassLayers(InClass);
	}
}

void USkeletalMeshComponent::UnlinkAnimClassLayers(TSubclassOf<UAnimInstance> InClass)
{
	if(bEnableAnimation && AnimScriptInstance)
	{
		AnimScriptInstance->UnlinkAnimClassLayers(InClass);
	}
}

UAnimInstance* USkeletalMeshComponent::GetLinkedAnimLayerInstanceByGroup(FName InGroup) const
{
	if(bEnableAnimation && AnimScriptInstance)
	{
		return AnimScriptInstance->GetLinkedAnimLayerInstanceByGroup(InGroup);
	}
	return nullptr;
}

UAnimInstance* USkeletalMeshComponent::GetLinkedAnimLayerInstanceByClass(TSubclassOf<UAnimInstance> InClass) const
{
	if(bEnableAnimation && AnimScriptInstance)
	{
		return AnimScriptInstance->GetLinkedAnimLayerInstanceByClass(InClass);
	}
	return nullptr;
}

void USkeletalMeshComponent::ForEachAnimInstance(TFunctionRef<void(UAnimInstance*)> InFunction)
{
	if(AnimScriptInstance)
	{
		InFunction(AnimScriptInstance);
	}

	// Copy LinkedInstances because the array can be concurrently modified inside this loop 
	for(UAnimInstance* LinkedInstance : TArray<TObjectPtr<UAnimInstance>, TInlineAllocator<8>>(LinkedInstances))
	{
		if(LinkedInstance)
		{
			InFunction(LinkedInstance);
		}
	}

	if(PostProcessAnimInstance)
	{
		InFunction(PostProcessAnimInstance);
	}
}

bool USkeletalMeshComponent::HasValidAnimationInstance() const
{
	return AnimScriptInstance || PostProcessAnimInstance;
}

void USkeletalMeshComponent::ResetAnimInstanceDynamics(ETeleportType InTeleportType)
{
	if(AnimScriptInstance)
	{
		AnimScriptInstance->ResetDynamics(InTeleportType);
	}

	for(UAnimInstance* LinkedInstance : LinkedInstances)
	{
		if (LinkedInstance)
		{
			LinkedInstance->ResetDynamics(InTeleportType);
		}
	}

	if(PostProcessAnimInstance)
	{
		PostProcessAnimInstance->ResetDynamics(InTeleportType);
	}
}

void USkeletalMeshComponent::NotifySkelControlBeyondLimit( USkelControlLookAt* LookAt ) {}

void USkeletalMeshComponent::SkelMeshCompOnParticleSystemFinished( UParticleSystemComponent* PSC )
{
	PSC->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
	PSC->UnregisterComponent();
}


void USkeletalMeshComponent::HideBone( int32 BoneIndex, EPhysBodyOp PhysBodyOption)
{
	Super::HideBone(BoneIndex, PhysBodyOption);

	if (!GetSkeletalMeshAsset())
	{
		return;
	}

	if (LeaderPoseComponent.IsValid())
	{
		return;
	}

	// if valid bone index
	if (BoneIndex >= 0 && GetNumBones() > BoneIndex)
	{
		bRequiredBonesUpToDate = false;

		if (PhysBodyOption != PBO_None)
		{
			FName HideBoneName = GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(BoneIndex);
			if (PhysBodyOption == PBO_Term)
			{
				TermBodiesBelow(HideBoneName);
			}
		}
	}
	else
	{
		UE_LOGF(LogSkeletalMesh, Warning, "HideBone[%ls]: Invalid Body Index (%d) has entered. This component doesn't contain buffer for the given body.", *GetPathNameSafe(GetSkeletalMeshAsset()), BoneIndex);
	}
}

void USkeletalMeshComponent::UnHideBone( int32 BoneIndex )
{
	Super::UnHideBone(BoneIndex);

	if (!GetSkeletalMeshAsset())
	{
		return;
	}

	if (LeaderPoseComponent.IsValid())
	{
		return;
	}

	if (BoneIndex >= 0 && GetNumBones() > BoneIndex)
	{
		bRequiredBonesUpToDate = false;

		//FName HideBoneName = GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(BoneIndex);
		// It's okay to turn this on for terminated bodies
		// It won't do any if BodyData isn't found
		// @JTODO
		//SetCollisionBelow(true, HideBoneName);
	}
	else
	{
		UE_LOGF(LogSkeletalMesh, Warning, "UnHideBone[%ls]: Invalid Body Index (%d) has entered. This component doesn't contain buffer for the given body.", *GetPathNameSafe(GetSkeletalMeshAsset()), BoneIndex);
	}
}


bool USkeletalMeshComponent::IsAnySimulatingPhysics() const
{
	for ( int32 BodyIndex=0; BodyIndex<Bodies.Num(); ++BodyIndex )
	{
		if (Bodies[BodyIndex]->IsInstanceSimulatingPhysics())
		{
			return true;
		}
	}

	return false;
}

void USkeletalMeshComponent::SetMorphTarget(FName MorphTargetName, float Value, bool bRemoveZeroWeight)
{
	float *CurveValPtr = MorphTargetCurves.Find(MorphTargetName);
	bool bShouldAddToList = !bRemoveZeroWeight || FPlatformMath::Abs(Value) > ZERO_ANIMWEIGHT_THRESH;
	if ( bShouldAddToList )
	{
		if ( CurveValPtr )
		{
			// sum up, in the future we might normalize, but for now this just sums up
			// this won't work well if all of them have full weight - i.e. additive 
			*CurveValPtr = Value;
		}
		else
		{
			MorphTargetCurves.Add(MorphTargetName, Value);
		}
	}
	// if less than ZERO_ANIMWEIGHT_THRESH
	// no reason to keep them on the list
	else 
	{
		// remove if found
		MorphTargetCurves.Remove(MorphTargetName);
	}
}

void USkeletalMeshComponent::ClearMorphTargets()
{
	MorphTargetCurves.Empty();
}

float USkeletalMeshComponent::GetMorphTarget( FName MorphTargetName ) const
{
	const float *CurveValPtr = MorphTargetCurves.Find(MorphTargetName);
	
	if(CurveValPtr)
	{
		return *CurveValPtr;
	}
	else
	{
		return 0.0f;
	}
}

FVector USkeletalMeshComponent::GetClosestCollidingRigidBodyLocation(const FVector& TestLocation) const
{
	float BestDistSq = UE_BIG_NUMBER;
	FVector Best = TestLocation;

	UPhysicsAsset* PhysicsAsset = GetPhysicsAsset();
	if( PhysicsAsset )
	{
		for (int32 i=0; i<Bodies.Num(); i++)
		{
			FBodyInstance* BodyInst = Bodies[i];
			if( BodyInst && BodyInst->IsValidBodyInstance() && (BodyInst->GetCollisionEnabled() != ECollisionEnabled::NoCollision) )
			{
				const FVector BodyLocation = BodyInst->GetUnrealWorldTransform().GetTranslation();
				const float DistSq = (BodyLocation - TestLocation).SizeSquared();
				if( DistSq < BestDistSq )
				{
					Best = BodyLocation;
					BestDistSq = DistSq;
				}
			}
		}
	}

	return Best;
}

void USkeletalMeshComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	for (int32 i=0; i < Bodies.Num(); ++i)
	{
		if (Bodies[i] != nullptr && Bodies[i]->IsValidBodyInstance())
		{
			Bodies[i]->GetBodyInstanceResourceSizeEx(CumulativeResourceSize);
		}
	}
}

#if UE_WITH_REMOTE_OBJECT_HANDLE

void USkeletalMeshComponent::MigratePhysicsState(FArchive& Ar)
{
	const FUObjectMigrationContext* MigrationContext = Ar.GetMigrationContext();
	check(MigrationContext);

	if (Ar.IsLoading())
	{
		Chaos::FSerializedDataBufferPtr MigratedStateData = MakeUnique<Chaos::FSerializedDataBuffer>();

		Ar << (*MigratedStateData);

		const EObjectMigrationRecvType ObjMigrationRecvType = MigrationContext->GetObjectMigrationRecvType(this);

		if (MigratedStateData->GetDataAsByteArrayRef().IsEmpty())
		{
			UE_LOGF(LogSkeletalMesh, Verbose, "Migrated an empty physics state (this can be expected) | [%ls] will use its default physics state or the one already had | Migration Context [%ls].", *GetPathNameSafe(this), ToString(ObjMigrationRecvType));
			return;
		}

		if (const bool bFinishedDestroy = HasAnyFlags(EObjectFlags::RF_MirroredGarbage))
		{
			return;
		}
		
		FPhysScene* PhysicsScene = FPhysicsInterface::GetCurrentSceneForComponent(this);
		if (ensure(PhysicsScene))
		{
			PhysicsScene->PushPendingSerializedStateForObject(this, MoveTemp(MigratedStateData));
		}
	}
	else if (Ar.IsSaving())
	{
		const EObjectMigrationSendType ObjMigrationSendType = MigrationContext->GetObjectMigrationSendType(this);

		if (CanSendMigratedPhysicsState(MigrationContext, ObjMigrationSendType))
		{
			Chaos::FSerializedMultiPhysicsState MigratedState(ObjMigrationSendType == EObjectMigrationSendType::ReassignOwnership ? Chaos::ESerializedDataContext::Both : Chaos::ESerializedDataContext::External);
			MigratedState.SetNumOfBodies(Bodies.Num());
			MigratedState.SetNumOfConstraints(Constraints.Num());

			for (FBodyInstance* PhysicsBody : Bodies)
			{
				check(PhysicsBody);
				MigratedState.AppendDataFromElement(*PhysicsBody);
			}
			
			for (FConstraintInstance* ConstraintInstance : Constraints)
			{
				check(ConstraintInstance);
				MigratedState.AppendDataFromElement(*ConstraintInstance);
			}

			// We need re-serialize the state as a chaos data buffer, so we can use it as is on de-serialization
			Chaos::FSerializedDataBuffer MigratedStateAsBytes;
			FMemoryWriter MigratedStateWriter(MigratedStateAsBytes.GetDataAsByteArrayRef());
			MigratedState.Serialize(MigratedStateWriter);

			Ar << MigratedStateAsBytes;
		}
		else
		{
			// This object might be being migrated before we had a chance to process a previous migration
			FPhysScene* PhysicsScene = FPhysicsInterface::GetCurrentSceneForComponent(this);
			Chaos::FSerializedDataBufferPtr RawMigratedData = PhysicsScene ? PhysicsScene->PopPendingSerializedStateForObject(this) : nullptr;
			if (!ensure(!RawMigratedData))
			{
				UE_LOGF(LogSkeletalMesh, Error, "[%s] Attempted to migrated an object physics state, before processing a previous pending physics state| Discarding pending physics state and sending an empty one...", __func__);
			}
			
			// We need to serialize an empty buffer as array, so we can handle this case during the load phase post migration
			Chaos::FSerializedDataBuffer EmptyPhysicsState;
			Ar << EmptyPhysicsState;
		}
	}
}

void USkeletalMeshComponent::ApplyMigratedPhysicsState(Chaos::FSerializedDataBufferPtr MigratedStateData)
{
	Chaos::FSerializedDataBufferPtr RawMigratedData;

	if (MigratedStateData)
	{
		RawMigratedData = MoveTemp(MigratedStateData);
	}
	else if (FPhysScene* PhysicsScene = FPhysicsInterface::GetCurrentSceneForComponent(this))
	{
		RawMigratedData = PhysicsScene->PopPendingSerializedStateForObject(this);
	}

	if (RawMigratedData)
	{
		FMemoryReader MultiBodyStateReader(RawMigratedData->GetDataAsByteArrayRef());

		Chaos::FSerializedMultiPhysicsState MigratedState;
		MigratedState.Serialize(MultiBodyStateReader);

		const Chaos::FSerializedMultiPhysicsState::FHeader& MigratedStateHeader = MigratedState.GetHeader();
		const bool bIsValidMigratedState = MigratedStateHeader.IsValid() && MigratedStateHeader.NumBodies == Bodies.Num() && MigratedStateHeader.NumConstraints == Constraints.Num();
		if (!ensure(bIsValidMigratedState))
		{
			return;
		}

		UE_LOGF(LogSkeletalMesh, Verbose, "[%s] Applying migrated physics state | Data context [%ls]", __func__, Chaos::LexToString( MigratedState.GetHeader().DataContext));

		for (FBodyInstance* BodyInstanceIt : Bodies)
		{
			check(BodyInstanceIt);
			MigratedState.ApplyDataToElement(*BodyInstanceIt);
		}

		for (FConstraintInstance* ConstraintInstance : Constraints)
		{
			check(ConstraintInstance);

			// Some other components may invalidates constraints in skeletal mesh to perform their own simulation (e.g. see UChaosWheeledVehicleMovementComponent::FixupSkeletalMesh())
			// This code ignores invalid constraint data on the receiving server to get around the ensure in Chaos::Private::ExtractJointConstraintDataFromBuffer
			auto ApplyMigratedData = [&MigratedState, ConstraintInstance](Chaos::ESerializedDataContext ContextToApply)
				{
					Chaos::FSerializedDataBufferPtr DataBuffer = MakeUnique<Chaos::FSerializedDataBuffer>();

					MigratedState.ReadElementDataIntoBuffer(*DataBuffer);

					if (!DataBuffer->GetDataAsByteArrayRef().IsEmpty())
					{
						ConstraintInstance->ApplySerializedState(MoveTemp(DataBuffer), ContextToApply);
					}
					else
					{
						UE_LOGF(LogSkeletalMesh, Log, "[%s]: Constraint %ls has empty data when migrated so it is skipped.",__func__, *ConstraintInstance->JointName.ToString());
					}
				};

			if (MigratedState.GetHeader().DataContext == Chaos::ESerializedDataContext::Both)
			{
				ApplyMigratedData(Chaos::ESerializedDataContext::Internal);
				ApplyMigratedData(Chaos::ESerializedDataContext::External);
			}
			else
			{
				ApplyMigratedData(MigratedState.GetHeader().DataContext);
			}
		}
	}
}

#endif

void USkeletalMeshComponent::SetAnimationMode(EAnimationMode::Type InAnimationMode, bool bForceInitAnimScriptInstance)
{
	if(!bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "SetAnimationMode: Animation is currently disabled");
		return;
	}

	const bool bNeedChange = AnimationMode != InAnimationMode;
	if (bNeedChange)
	{
		AnimationMode = InAnimationMode;
		ClearAnimScriptInstance();
	}

	// when mode is swapped, make sure to reinitialize
	// even if it was same mode, this was due to users who wants to use BP construction script to do this
	// if you use it in the construction script, it gets serialized, but it never instantiate. 
	if(GetSkeletalMeshAsset() != nullptr && (bNeedChange || (AnimationMode == EAnimationMode::AnimationBlueprint && bForceInitAnimScriptInstance)))
	{
		if (InitializeAnimScriptInstance(true))
		{
			OnAnimInitialized.Broadcast();
		}
	}
}

EAnimationMode::Type USkeletalMeshComponent::GetAnimationMode() const
{
	return AnimationMode;
}

void USkeletalMeshComponent::PlayAnimation(class UAnimationAsset* NewAnimToPlay, bool bLooping)
{
	if(!bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "PlayAnimation: Animation is currently disabled");
		return;
	}

	SetAnimationMode(EAnimationMode::AnimationSingleNode);
	SetAnimation(NewAnimToPlay);
	Play(bLooping);
}

void USkeletalMeshComponent::SetAnimation(UAnimationAsset* NewAnimToPlay)
{
	if(!bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "SetAnimation: Animation is currently disabled");
		return;
	}

	UAnimSingleNodeInstance* SingleNodeInstance = GetSingleNodeInstance();
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetAnimationAsset(NewAnimToPlay, false);
		SingleNodeInstance->SetPlaying(false);
	}
	else if( AnimScriptInstance != nullptr )
	{
		UE_LOGF(LogAnimation, Warning, "Currently in Animation Blueprint mode. Please change AnimationMode to Use Animation Asset");
	}
}

void USkeletalMeshComponent::Play(bool bLooping)
{
	if(!bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "Play: Animation is currently disabled");
		return;
	}

	UAnimSingleNodeInstance* SingleNodeInstance = GetSingleNodeInstance();
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetPlaying(true);
		SingleNodeInstance->SetLooping(bLooping);
	}
	else if( AnimScriptInstance != nullptr )
	{
		UE_LOGF(LogAnimation, Warning, "Currently in Animation Blueprint mode. Please change AnimationMode to Use Animation Asset");
	}
}

void USkeletalMeshComponent::Stop()
{
	if(!bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "Stop: Animation is currently disabled");
		return;
	}

	UAnimSingleNodeInstance* SingleNodeInstance = GetSingleNodeInstance();
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetPlaying(false);
	}
	else if( AnimScriptInstance != nullptr )
	{
		UE_LOGF(LogAnimation, Warning, "Currently in Animation Blueprint mode. Please change AnimationMode to Use Animation Asset");
	}
}

bool USkeletalMeshComponent::IsPlaying() const
{
	UAnimSingleNodeInstance* SingleNodeInstance = GetSingleNodeInstance();
	if (SingleNodeInstance)
	{
		return SingleNodeInstance->IsPlaying();
	}
	else if( AnimScriptInstance != nullptr )
	{
		UE_LOGF(LogAnimation, Warning, "Currently in Animation Blueprint mode. Please change AnimationMode to Use Animation Asset");
	}

	return false;
}

void USkeletalMeshComponent::SetPosition(float InPos, bool bFireNotifies)
{
	if(!bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "SetPosition: Animation is currently disabled");
		return;
	}

	UAnimSingleNodeInstance* SingleNodeInstance = GetSingleNodeInstance();
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetPosition(InPos, bFireNotifies);
	}
	else if( AnimScriptInstance != nullptr )
	{
		UE_LOGF(LogAnimation, Warning, "Currently in Animation Blueprint mode. Please change AnimationMode to Use Animation Asset");
	}
}

float USkeletalMeshComponent::GetPosition() const
{
	UAnimSingleNodeInstance* SingleNodeInstance = GetSingleNodeInstance();
	if (SingleNodeInstance)
	{
		return SingleNodeInstance->GetCurrentTime();
	}
	else if( AnimScriptInstance != nullptr )
	{
		UE_LOGF(LogAnimation, Warning, "Currently in Animation Blueprint mode. Please change AnimationMode to Use Animation Asset");
	}

	return 0.f;
}

void USkeletalMeshComponent::SetPlayRate(float Rate)
{
	if(!bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "SetPlayRate: Animation is currently disabled");
		return;
	}

	UAnimSingleNodeInstance* SingleNodeInstance = GetSingleNodeInstance();
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetPlayRate(Rate);
	}
	else if( AnimScriptInstance != nullptr )
	{
		UE_LOGF(LogAnimation, Warning, "Currently in Animation Blueprint mode. Please change AnimationMode to Use Animation Asset");
	}
}

float USkeletalMeshComponent::GetPlayRate() const
{
	UAnimSingleNodeInstance* SingleNodeInstance = GetSingleNodeInstance();
	if (SingleNodeInstance)
	{
		return SingleNodeInstance->GetPlayRate();
	}
	else if( AnimScriptInstance != nullptr )
	{
		UE_LOGF(LogAnimation, Warning, "Currently in Animation Blueprint mode. Please change AnimationMode to Use Animation Asset");
	}

	return 0.f;
}

void USkeletalMeshComponent::OverrideAnimationData(UAnimationAsset* InAnimToPlay, bool bIsLooping /*= true*/, bool bIsPlaying /*= true*/, float Position /*= 0.f*/, float PlayRate /*= 1.f*/)
{
	if(!bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "OverrideAnimationData: Animation is currently disabled");
		return;
	}

	AnimationData.AnimToPlay = InAnimToPlay;
	AnimationData.bSavedLooping = bIsLooping;
	AnimationData.bSavedPlaying = bIsPlaying;
	AnimationData.SavedPosition = Position;
	AnimationData.SavedPlayRate = PlayRate;
	SetAnimationMode(EAnimationMode::AnimationSingleNode);
	TickAnimation(0.f, false);
	RefreshBoneTransforms();
}

class UAnimSingleNodeInstance* USkeletalMeshComponent::GetSingleNodeInstance() const
{
	return Cast<class UAnimSingleNodeInstance>(AnimScriptInstance);
}

bool USkeletalMeshComponent::PoseTickedThisFrame() const 
{ 
	return GFrameCounter == LastPoseTickFrame; 
}

FTransform USkeletalMeshComponent::ConvertLocalRootMotionToWorld(const FTransform& InTransform)
{
	// Make sure component to world is up to date
	ConditionalUpdateComponentToWorld();

#if !(UE_BUILD_SHIPPING)
	if (GetComponentTransform().ContainsNaN())
	{
		logOrEnsureNanError(TEXT("SkeletalMeshComponent: GetComponentTransform() contains NaN!"));
		SetComponentToWorld(FTransform::Identity);
	}
#endif

	//Calculate new actor transform after applying root motion to this component
	const FTransform ActorToWorld = GetOwner()->GetTransform();

	const FTransform ComponentToActor = ActorToWorld.GetRelativeTransform(GetComponentTransform());
	const FTransform NewComponentToWorld = InTransform * GetComponentTransform();
	const FTransform NewActorTransform = ComponentToActor * NewComponentToWorld;

	const FVector DeltaWorldTranslation = NewActorTransform.GetTranslation() - ActorToWorld.GetTranslation();

	const FQuat NewWorldRotation = GetComponentTransform().GetRotation() * InTransform.GetRotation();
	const FQuat DeltaWorldRotation = NewWorldRotation * GetComponentTransform().GetRotation().Inverse();
	
	const FTransform DeltaWorldTransform(DeltaWorldRotation, DeltaWorldTranslation);

	UE_LOGF(LogRootMotion, Log,  "ConvertLocalRootMotionToWorld LocalT: %ls, LocalR: %ls, WorldT: %ls, WorldR: %ls.",
		*InTransform.GetTranslation().ToCompactString(), *InTransform.GetRotation().Rotator().ToCompactString(),
		*DeltaWorldTransform.GetTranslation().ToCompactString(), *DeltaWorldTransform.GetRotation().Rotator().ToCompactString());

	return DeltaWorldTransform;
}

FRootMotionMovementParams USkeletalMeshComponent::ConsumeRootMotion()
{
	float InterpAlpha;
	
	if(bExternalTickRateControlled)
	{
		InterpAlpha = ExternalInterpolationAlpha;
	}
	else
	{
		InterpAlpha = ShouldUseUpdateRateOptimizations() ? AnimUpdateRateParams->GetRootMotionInterp() : 1.f;
	}

	return ConsumeRootMotion_Internal(InterpAlpha);
}

#if WITH_EDITOR
FPoseWatchDynamicData::FPoseWatchDynamicData(USkeletalMeshComponent* InComponent)
{
	check(InComponent);
	if (InComponent->GetSkeletalMeshAsset())
	{
		PoseWatches = InComponent->PoseWatches;

		for (FAnimNodePoseWatch& PoseWatch : PoseWatches)
		{
			PoseWatch.CopyPoseWatchData(InComponent->GetSkeletalMeshAsset()->GetRefSkeleton());
		}
	}
}
#endif

FRootMotionMovementParams USkeletalMeshComponent::ConsumeRootMotion_Internal(float InAlpha)
{
	FRootMotionMovementParams RootMotion;
	if(AnimScriptInstance)
	{
		RootMotion.Accumulate(AnimScriptInstance->ConsumeExtractedRootMotion(InAlpha));

		for(UAnimInstance* LinkedInstance : LinkedInstances)
		{
			RootMotion.Accumulate(LinkedInstance->ConsumeExtractedRootMotion(InAlpha));
		}
	}

	if(PostProcessAnimInstance)
	{
		RootMotion.Accumulate(PostProcessAnimInstance->ConsumeExtractedRootMotion(InAlpha));
	}

	return RootMotion;
}

float USkeletalMeshComponent::CalculateMass(FName BoneName)
{
	float Mass = 0.0f;

	if (Bodies.Num())
	{
		for (int32 i = 0; i < Bodies.Num(); ++i)
		{
			UBodySetup* BodySetupPtr = Bodies[i]->GetBodySetup();
			//if bone name is not provided calculate entire mass - otherwise get mass for just the bone
			if (BodySetupPtr && (BoneName == NAME_None || BoneName == BodySetupPtr->BoneName))
			{
				Mass += BodySetupPtr->CalculateMass(this);
			}
		}
	}
	else	//We want to calculate mass before we've initialized body instances - in this case use physics asset setup
	{
		using BodySetupContainerType = decltype(UPhysicsAsset::SkeletalBodySetups);
		BodySetupContainerType* BodySetups = nullptr;
		if (UPhysicsAsset* PhysicsAsset = GetPhysicsAsset())
		{
			BodySetups = &PhysicsAsset->SkeletalBodySetups;
		}

		if (BodySetups)
		{
			for (int32 i = 0; i < BodySetups->Num(); ++i)
			{
				if ((*BodySetups)[i] && (BoneName == NAME_None || BoneName == (*BodySetups)[i]->BoneName))
				{
					Mass += (*BodySetups)[i]->CalculateMass(this);
				}
			}
		}
	}

	return Mass;
}

bool USkeletalMeshComponent::IsShown(const FEngineShowFlags& ShowFlags) const
{
	return ShowFlags.SkeletalMeshes;
}

#if WITH_EDITOR

bool USkeletalMeshComponent::ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	if (!bConsiderOnlyBSP && MeshObject != nullptr)
	{
		FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();
		if (SkelMeshRenderData && SkelMeshRenderData->LODRenderData.Num() > 0)
		{
			// Transform verts into world space. Note that this assumes skeletal mesh is in reference pose...
			const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[0];
			// When editor accesses cooked data and mesh streaming is on, lower LODs are cooked as StreamingBulkData instead of cooking inline,
			// so the vertex buffer may contain valid meta data but no actual vertex data. Check it is valid before accessing it.
			if (LODData.StaticVertexBuffers.PositionVertexBuffer.GetVertexData())
			{
				for (uint32 VertIdx = 0; VertIdx < LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices(); VertIdx++)
				{
					const FVector3f& VertexPos(LODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertIdx));
					const FVector Location = GetComponentTransform().TransformPosition((FVector)VertexPos);
					const bool bLocationIntersected = FMath::PointBoxIntersection(Location, InSelBBox);

					// If the selection box doesn't have to encompass the entire component and a skeletal mesh vertex has intersected with
					// the selection box, this component is being touched by the selection box
					if (!bMustEncompassEntireComponent && bLocationIntersected)
					{
						return true;
					}

					// If the selection box has to encompass the entire component and a skeletal mesh vertex didn't intersect with the selection
					// box, this component does not qualify
					else if (bMustEncompassEntireComponent && !bLocationIntersected)
					{
						return false;
					}
				}	

				// if bMustEncompassEntireComponent == false, we require at least one intersection, so if we are here, no intersection happened
				// if bMustEncompassEntireComponent == true, all of them intersected, else the for loop would have returned false already
				if (bMustEncompassEntireComponent)
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool USkeletalMeshComponent::ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	if (!bConsiderOnlyBSP && MeshObject != nullptr)
	{
		FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();
		if (SkelMeshRenderData && SkelMeshRenderData->LODRenderData.Num() > 0)
		{
			// Transform verts into world space. Note that this assumes skeletal mesh is in reference pose...
			const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[0];
			// When editor accesses cooked data and mesh streaming is on, lower LODs are cooked as StreamingBulkData instead of cooking inline,
			// so the vertex buffer may contain valid meta data but no actual vertex data. Check it is valid before accessing it.
			if (LODData.StaticVertexBuffers.PositionVertexBuffer.GetVertexData())
			{
				for (uint32 VertIdx = 0; VertIdx < LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices(); VertIdx++)
				{
					const FVector3f& VertexPos(LODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertIdx));
					const FVector Location = GetComponentTransform().TransformPosition((FVector)VertexPos);
					const bool bLocationIntersected = InFrustum.IntersectSphere(Location, 0.0f);

					// If the selection box doesn't have to encompass the entire component and a skeletal mesh vertex has intersected with
					// the selection box, this component is being touched by the selection box
					if (!bMustEncompassEntireComponent && bLocationIntersected)
					{
						return true;
					}

					// If the selection box has to encompass the entire component and a skeletal mesh vertex didn't intersect with the selection
					// box, this component does not qualify
					else if (bMustEncompassEntireComponent && !bLocationIntersected)
					{
						return false;
					}
				}

				// if bMustEncompassEntireComponent == false, we require at least one intersection, so if we are here, no intersection happened
				// if bMustEncompassEntireComponent == true, all of them intersected, else the for loop would have returned false already
				if (bMustEncompassEntireComponent)
				{
					return true;
				}
			}
		}
	}

	return false;
}


void USkeletalMeshComponent::UpdateCollisionProfile()
{
	Super::UpdateCollisionProfile();

	for(int32 i=0; i < Bodies.Num(); ++i)
	{
		if(Bodies[i]->BodySetup.IsValid())
		{
			Bodies[i]->LoadProfileData(false);
		}
	}
}

FDelegateHandle USkeletalMeshComponent::RegisterOnSkeletalMeshPropertyChanged( const FOnSkeletalMeshPropertyChanged& Delegate )
{
	return OnSkeletalMeshPropertyChanged.Add(Delegate);
}

void USkeletalMeshComponent::UnregisterOnSkeletalMeshPropertyChanged( FDelegateHandle Handle )
{
	OnSkeletalMeshPropertyChanged.Remove(Handle);
}

void USkeletalMeshComponent::ValidateAnimation()
{
	USkeletalMesh* SkelMesh = GetSkeletalMeshAsset();
	if (SkelMesh && SkelMesh->GetSkeleton() == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "SkeletalMesh %ls has no skeleton. This needs to fixed before an animation can be set", *SkelMesh->GetFullName());
		if (AnimationMode == EAnimationMode::AnimationSingleNode)
		{
			AnimationData.AnimToPlay = nullptr;
		}
		else if(AnimationMode == EAnimationMode::AnimationBlueprint)
		{
			AnimClass = nullptr;
		}
		else
		{
			// if custom mode, you still can't use the animation instance
			AnimScriptInstance = nullptr;
		}
		return;
	}

	if(AnimationMode == EAnimationMode::AnimationSingleNode)
	{
		if (AnimationData.AnimToPlay && SkelMesh)
		{
			if (AnimationData.AnimToPlay->GetSkeleton() == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "Animation %ls is incompatible because it has no skeleton, removing animation from actor.", *AnimationData.AnimToPlay->GetName());
				AnimationData.AnimToPlay = nullptr;
			}
		}
	}
	else if (AnimationMode == EAnimationMode::AnimationBlueprint)
	{
		IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(AnimClass);
		if (AnimClassInterface && SkelMesh)
		{
			if (SkelMesh->GetSkeleton() == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "AnimBP %ls is incompatible because mesh %ls has no skeleton, removing AnimBP from actor.", *AnimClass->GetName(), *SkelMesh->GetName());
				AnimClass = nullptr;
			}
		}
	}
}

#endif

bool USkeletalMeshComponent::IsPlayingRootMotion() const
{
	return (IsPlayingRootMotionFromEverything() || IsPlayingNetworkedRootMotionMontage());
}

bool USkeletalMeshComponent::IsPlayingNetworkedRootMotionMontage() const
{
	if (AnimScriptInstance)
	{
		if (AnimScriptInstance->RootMotionMode == ERootMotionMode::RootMotionFromMontagesOnly)
		{
			if (const FAnimMontageInstance* MontageInstance = AnimScriptInstance->GetRootMotionMontageInstance())
			{
				return !MontageInstance->IsRootMotionDisabled();
			}
		}
	}

	return false;
}

bool USkeletalMeshComponent::IsPlayingRootMotionFromEverything() const
{
	return AnimScriptInstance && (AnimScriptInstance->RootMotionMode == ERootMotionMode::RootMotionFromEverything);
}

void USkeletalMeshComponent::ResetRootBodyIndex()
{
	RootBodyData.BodyIndex = INDEX_NONE;
	RootBodyData.TransformToRoot = FTransform::Identity;
}

void USkeletalMeshComponent::SetRootBodyIndex(int32 InBodyIndex)
{
	// this is getting called prior to initialization. 
	// @todo : better fix is to initialize it? overkilling it though. 
	if (InBodyIndex != INDEX_NONE)
	{
		RootBodyData.BodyIndex = InBodyIndex;
		RootBodyData.TransformToRoot = FTransform::Identity;

		// Only need to do further work if we have any bodies at all (ie physics state is created)
		if (Bodies.Num() > 0)
		{
			if (Bodies.IsValidIndex(RootBodyData.BodyIndex))
			{
				FBodyInstance* BI = Bodies[RootBodyData.BodyIndex];
				RootBodyData.TransformToRoot = GetComponentToWorld().GetRelativeTransform(BI->GetUnrealWorldTransform());
			}
			else
			{
				ResetRootBodyIndex();
			}
		}
	}
}

void USkeletalMeshComponent::RefreshMorphTargets()
{
	ResetMorphTargetCurves();

	if (GetSkeletalMeshAsset() && AnimScriptInstance)
	{
		// as this can be called from any worker thread (i.e. from CreateRenderState_Concurrent) we cant currently be doing parallel evaluation
		check(!IsRunningParallelEvaluation());
		AnimScriptInstance->RefreshCurves(this);

		for(UAnimInstance* LinkedInstance : LinkedInstances)
		{
			LinkedInstance->RefreshCurves(this);
		}
		
		if(PostProcessAnimInstance)
		{
			PostProcessAnimInstance->RefreshCurves(this);
		}
	}
	else if (USkeletalMeshComponent* LeaderSMC = Cast<USkeletalMeshComponent>(LeaderPoseComponent.Get()))
	{
		if (LeaderSMC->AnimScriptInstance)
		{
			LeaderSMC->AnimScriptInstance->RefreshCurves(this);
		}
	}

	UpdateMorphTargetOverrideCurves();
}

void USkeletalMeshComponent::SendRenderDynamicData_Concurrent()
{
#if WITH_STATE_STREAM_ACTOR
	if (true)
		return;
#endif

	Super::SendRenderDynamicData_Concurrent();

#if WITH_EDITOR
	if (GetSkeletalMeshAsset() && GetSkeletalMeshAsset()->IsCompiling())
	{
		return;
	}

	if (SceneProxy && !SceneProxy->IsNaniteMesh()) // TODO: Nanite-Skinning
	{
		UpdatePoseWatches();
		FPoseWatchDynamicData* NewDynamicData = new FPoseWatchDynamicData(this);

		FSkeletalMeshSceneProxy* TargetProxy = (FSkeletalMeshSceneProxy*)SceneProxy;

		ENQUEUE_RENDER_COMMAND(PoseWatchDynamicDataCommand)(UE::RenderCommandPipe::SkeletalMesh,
			[TargetProxy, NewDynamicData]
			{
				if (TargetProxy->PoseWatchDynamicData)
				{
					delete TargetProxy->PoseWatchDynamicData;
				}

				TargetProxy->PoseWatchDynamicData = NewDynamicData;
			}
		);
	} //-V773
#endif
}

void USkeletalMeshComponent::ParallelAnimationEvaluation() 
{
	if (AnimEvaluationContext.bDoInterpolation)
	{
		PerformAnimationProcessing(AnimEvaluationContext.SkeletalMesh, AnimEvaluationContext.AnimInstance, AnimEvaluationContext.bDoEvaluation, AnimEvaluationContext.bForceRefPose, AnimEvaluationContext.CachedComponentSpaceTransforms, AnimEvaluationContext.CachedBoneSpaceTransforms, AnimEvaluationContext.RootBoneTranslation, AnimEvaluationContext.CachedCurve, AnimEvaluationContext.CachedCustomAttributes);
	}
	else
	{
		PerformAnimationProcessing(AnimEvaluationContext.SkeletalMesh, AnimEvaluationContext.AnimInstance, AnimEvaluationContext.bDoEvaluation, AnimEvaluationContext.bForceRefPose, AnimEvaluationContext.ComponentSpaceTransforms, AnimEvaluationContext.BoneSpaceTransforms, AnimEvaluationContext.RootBoneTranslation, AnimEvaluationContext.Curve, AnimEvaluationContext.CustomAttributes);
	}

	ParallelDuplicateAndInterpolate(AnimEvaluationContext);

	if(AnimEvaluationContext.bDoEvaluation || AnimEvaluationContext.bDoInterpolation)
	{
		if(AnimEvaluationContext.AnimInstance)
		{
			AnimEvaluationContext.AnimInstance->UpdateCurvesToEvaluationContext(AnimEvaluationContext);
		}
		else if(AnimEvaluationContext.PostProcessAnimInstance)
		{
			AnimEvaluationContext.PostProcessAnimInstance->UpdateCurvesToEvaluationContext(AnimEvaluationContext);
		}
	}
}

void USkeletalMeshComponent::ParallelDuplicateAndInterpolate(FAnimationEvaluationContext& InAnimEvaluationContext)
{
	if(CVarUseParallelAnimationInterpolation.GetValueOnAnyThread() != 0)
	{
		if (InAnimEvaluationContext.bDuplicateToCacheCurve)
		{
			InAnimEvaluationContext.CachedCurve.CopyFrom(InAnimEvaluationContext.Curve);
		}

		if (InAnimEvaluationContext.bDuplicateToCachedAttributes)
		{
			InAnimEvaluationContext.CachedCustomAttributes.CopyFrom(InAnimEvaluationContext.CustomAttributes);
		}

		if (InAnimEvaluationContext.bDuplicateToCacheBones)
		{
			InAnimEvaluationContext.CachedComponentSpaceTransforms.Reset();
			InAnimEvaluationContext.CachedComponentSpaceTransforms.Append(InAnimEvaluationContext.ComponentSpaceTransforms);
			InAnimEvaluationContext.CachedBoneSpaceTransforms.Reset();
			InAnimEvaluationContext.CachedBoneSpaceTransforms.Append(InAnimEvaluationContext.BoneSpaceTransforms);
		}

		if (InAnimEvaluationContext.bDoInterpolation)
		{
			SCOPE_CYCLE_COUNTER(STAT_InterpolateSkippedFrames);

			float Alpha;
			if(bEnableUpdateRateOptimizations && AnimUpdateRateParams)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (AnimScriptInstance)
				{
					AnimScriptInstance->OnUROPreInterpolation();
					AnimScriptInstance->OnUROPreInterpolation_AnyThread(InAnimEvaluationContext);
				}

				for(UAnimInstance* LinkedInstance : LinkedInstances)
				{
					LinkedInstance->OnUROPreInterpolation();
					LinkedInstance->OnUROPreInterpolation_AnyThread(InAnimEvaluationContext);
				}

				if(PostProcessAnimInstance)
				{
					PostProcessAnimInstance->OnUROPreInterpolation();
					PostProcessAnimInstance->OnUROPreInterpolation_AnyThread(InAnimEvaluationContext);
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				Alpha = AnimUpdateRateParams->GetInterpolationAlpha();
			}
			else
			{
				Alpha = ExternalInterpolationAlpha;
			}

			FAnimationRuntime::LerpBoneTransforms(InAnimEvaluationContext.BoneSpaceTransforms, InAnimEvaluationContext.CachedBoneSpaceTransforms, Alpha, RequiredBones);
			InAnimEvaluationContext.SkeletalMesh->FillComponentSpaceTransforms(InAnimEvaluationContext.BoneSpaceTransforms, FillComponentSpaceTransformsRequiredBones, InAnimEvaluationContext.ComponentSpaceTransforms);

			// interpolate curve
			InAnimEvaluationContext.Curve.LerpTo(InAnimEvaluationContext.CachedCurve, Alpha);

			UE::Anim::Attributes::InterpolateAttributes(InAnimEvaluationContext.CustomAttributes, InAnimEvaluationContext.CachedCustomAttributes, Alpha);
		}
	}
}

void USkeletalMeshComponent::CompleteParallelAnimationEvaluation(bool bDoPostAnimEvaluation)
{
	SCOPED_NAMED_EVENT(USkeletalMeshComponent_CompleteParallelAnimationEvaluation, FColor::Yellow);
	ParallelAnimationEvaluationTask.SafeRelease(); //We are done with this task now, clean up!

	if (bDoPostAnimEvaluation && (AnimEvaluationContext.AnimInstance == AnimScriptInstance) && (AnimEvaluationContext.SkeletalMesh == GetSkeletalMeshAsset()) && (AnimEvaluationContext.ComponentSpaceTransforms.Num() == GetNumComponentSpaceTransforms()))
	{
		SwapEvaluationContextBuffers();

		PostAnimEvaluation(AnimEvaluationContext);
	}
	
	AnimEvaluationContext.Clear();
}

bool USkeletalMeshComponent::HandleExistingParallelEvaluationTask(bool bBlockOnTask, bool bPerformPostAnimEvaluation)
{
	// We are already processing eval and we aren't currently executing it on this thread
	if (IsRunningParallelEvaluation() && ParallelAnimationEvaluationTask->IsAwaitable())
	{
		if (bBlockOnTask)
		{
			// If we try and complete existing parallel evaluation in a transaction, we have to abort the transaction
			// and retry up to this point. This is because we can only have started parallel evaluation outside of a
			// transaction, and thus it is only safe to complete the work outside of a transaction too! For instance
			// if the parallel animation evaluation task depended on something that was trying to take a mutex that we
			// had also taken in the transaction, we'd have caused a deadlock if we didn't have this here.
			if (AutoRTFM::IsTransactional())
			{
				(void)AutoRTFM::Close([&]
					{
						AutoRTFM::OnRetry([strong_this = TObjectPtr<USkeletalMeshComponent>(this), bPerformPostAnimEvaluation]
							{
								UE_LOGF(LogSkeletalMesh, Display, "A call to `HandleExistingParallelEvaluationTask` that is blocking is causing Verse to abort and retry.");
								const bool bBlockOnTask = true; // Was true from the if block above!
								strong_this->HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation);
								UE_LOGF(LogSkeletalMesh, Display, "`HandleExistingParallelEvaluationTask` has completed after Verse aborted, and we are now retrying.");
							});
						AutoRTFM::CascadingRetryTransaction();
					});
				return true;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::BlockOnParallelEvaluationTask);
			check(IsInGameThread()); // Only attempt this from game thread!
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(ParallelAnimationEvaluationTask, ENamedThreads::GameThread);

			// We had a valid task handle and now we are done waiting for it
			// Perform completion now
			// Note that because completion happens in a separate task (on the main thread) from the one we waited on
			// it could execute after this call to finalize evaluation. If it does, it should be a no-op operation as
			// it means the skeletal mesh or some other core piece has changed and the completion validates that the
			// data we started with is consistent with the current data.
			// We don't wait for the main thread completion task because it could execute much later, instead we do it inline
			CompleteParallelAnimationEvaluation(bPerformPostAnimEvaluation);
		}

		return true;
	}

	return false;
}

void USkeletalMeshComponent::SuspendClothingSimulation()
{
	bClothingSimulationSuspended = true;
}

void USkeletalMeshComponent::ResumeClothingSimulation()
{
	bClothingSimulationSuspended = false;
	ForceClothNextUpdateTeleport();
}

bool USkeletalMeshComponent::IsClothingSimulationSuspended() const
{
	return bClothingSimulationSuspended;
}

void USkeletalMeshComponent::BindClothToLeaderPoseComponent()
{
	if(USkeletalMeshComponent* LeaderComp = Cast<USkeletalMeshComponent>(LeaderPoseComponent.Get()))
	{
		if(GetSkeletalMeshAsset() != LeaderComp->GetSkeletalMeshAsset())
		{
			// Not the same mesh, can't bind
			return;
		}

		if(ClothingSimulationInstances.Num() && LeaderComp->ClothingSimulationInstances.Num())
		{
			bDisableClothSimulation = true;

			// When we extract positions from now we'll just take the Leader components positions
			bBindClothToLeaderComponent = true;
		}
	}
}

void USkeletalMeshComponent::UnbindClothFromLeaderPoseComponent(bool bRestoreSimulationSpace)
{
	USkeletalMeshComponent* LeaderComp = Cast<USkeletalMeshComponent>(LeaderPoseComponent.Get());
	if(LeaderComp && bBindClothToLeaderComponent)
	{
		if(ClothingSimulationInstances.Num())
		{
			bDisableClothSimulation = false;
		}

		bBindClothToLeaderComponent = false;
	}
}

void USkeletalMeshComponent::ReinitializeAnimNodes()
{
	if (bRegistered && GetSkeletalMeshAsset() != nullptr)
	{
		if (AnimScriptInstance)
		{
			AnimScriptInstance->InitializeAnimation();
		}
		if (PostProcessAnimInstance)
		{
			PostProcessAnimInstance->InitializeAnimation();
		}
	}
}

void USkeletalMeshComponent::SetAllowRigidBodyAnimNode(bool bInAllow, bool bReinitAnim)
{
	if (bDisableRigidBodyAnimNode == bInAllow)
	{
		bDisableRigidBodyAnimNode = !bInAllow;

		if(bReinitAnim)
		{
			// need to reinitialize rigid body nodes for new setting to take effect
			ReinitializeAnimNodes();
		}
	}
}

void USkeletalMeshComponent::SetAllowAnimDynamicsAnimNode(bool bInAllow, bool bReinitAnim)
{
	if (bDisableAnimDynamicsAnimNode == bInAllow)
	{
		bDisableAnimDynamicsAnimNode = !bInAllow;

		if (bReinitAnim)
		{
			// need to reinitialize rigid body nodes for new setting to take effect
			ReinitializeAnimNodes();
		}
	}
}

void USkeletalMeshComponent::SetAllowClothActors(bool bInAllow)
{
	bAllowClothActors = bInAllow;
}

bool USkeletalMeshComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	UPhysicsAsset* PhysicsAsset = GetPhysicsAsset();
	if (PhysicsAsset && GetComponentTransform().GetScale3D().IsUniform())
	{
		const int32 MaxBodies = PhysicsAsset->SkeletalBodySetups.Num();
		for (int32 Idx = 0; Idx < MaxBodies; Idx++)
		{
			UBodySetup* const BS = PhysicsAsset->SkeletalBodySetups[Idx];
			int32 const BoneIndex = BS ? GetBoneIndex(BS->BoneName) : INDEX_NONE;

			if (BoneIndex != INDEX_NONE)
			{
				FTransform WorldBoneTransform = GetBoneTransform(BoneIndex, GetComponentTransform());
				if (FMath::Abs(WorldBoneTransform.GetDeterminant()) > (float)UE_KINDA_SMALL_NUMBER)
				{
					GeomExport.ExportRigidBodySetup(*BS, WorldBoneTransform);
				}
			}
		}
	}

	// skip fallback export of body setup data
	return false;
}

void USkeletalMeshComponent::FinalizeBoneTransform() 
{
	Super::FinalizeBoneTransform();

	if (UE::Anim::bUseSafeMeshPoseIndices && bValidMeshPoseTransformsUpToDate == false)
	{
		ComputeValidMeshPoseTransforms(FillComponentSpaceTransformsRequiredBones, GetMutableValidMeshPoseTransforms());
		bValidMeshPoseTransformsUpToDate = true;
	}

	// After pose has been finalized, dispatch AnimNotifyEvents in case they want to use up to date pose.
	// (For example attaching particle systems to up to date sockets).

	/////////////////////////////////////////////////////////////////////////////
	// Notify / Event Handling!
	// This can do anything to our component (including destroy it) 
	// Any code added after this point needs to take that into account
	/////////////////////////////////////////////////////////////////////////////

	ConditionallyDispatchQueuedAnimEvents();

	OnBoneTransformsFinalizedMC.Broadcast();

	TRACE_SKELETAL_MESH_COMPONENT(this);
}

bool USkeletalMeshComponent::ShouldEvaluatePostProcessAnimBP() const
{
	const int32 LODLevel = GetPredictedLODLevel();
	if (!(PostProcessAnimBPLODThreshold == INDEX_NONE || LODLevel <= PostProcessAnimBPLODThreshold))
	{
		return false;
	}

	if (USkeletalMesh* SkelMesh = GetSkeletalMeshAsset())
	{
		if (!SkelMesh->ShouldEvaluatePostProcessAnimGraph(LODLevel))
		{
			return false;
		}
	}

	return true;
}

bool USkeletalMeshComponent::ShouldUpdatePostProcessInstance() const
{
	if (!ShouldEvaluatePostProcessAnimBP())
	{
		return false;
	}

	return PostProcessAnimInstance && !bDisablePostProcessBlueprint;
}

bool USkeletalMeshComponent::ShouldPostUpdatePostProcessInstance() const
{
	if (!ShouldEvaluatePostProcessAnimBP())
	{
		return false;
	}

	return PostProcessAnimInstance && PostProcessAnimInstance->NeedsUpdate() && !bDisablePostProcessBlueprint;
}

void USkeletalMeshComponent::SetOverridePostProcessAnimBP(TSubclassOf<UAnimInstance> InPostProcessAnimBlueprint, bool ReinitAnimInstances)
{
	OverridePostProcessAnimBP = InPostProcessAnimBlueprint;

	if (ReinitAnimInstances)
	{
		InitializeAnimScriptInstance();
	}
}

TSubclassOf<UAnimInstance> USkeletalMeshComponent::GetPostProcessAnimBPClassToBeUsed() const
{
	if (OverridePostProcessAnimBP)
	{
		return OverridePostProcessAnimBP;
	}

	if (USkeletalMesh* SkelMesh = GetSkeletalMeshAsset())
	{
		return SkelMesh->GetPostProcessAnimBlueprint();
	}

	return nullptr;
}

bool USkeletalMeshComponent::ShouldEvaluatePostProcessInstance() const
{
	if (!OverridePostProcessAnimBP)
	{
		if (!ShouldEvaluatePostProcessAnimBP())
		{
			return false;
		}
	}

	return PostProcessAnimInstance && !bDisablePostProcessBlueprint;
}

void USkeletalMeshComponent::SetRefPoseOverride(const TArray<FTransform>& NewRefPoseTransforms)
{
	Super::SetRefPoseOverride(NewRefPoseTransforms);
	bRequiredBonesUpToDate = false;
}

void USkeletalMeshComponent::ClearRefPoseOverride()
{
	Super::ClearRefPoseOverride();
	bRequiredBonesUpToDate = false;
}

FDelegateHandle USkeletalMeshComponent::RegisterOnPhysicsCreatedDelegate(const FOnSkelMeshPhysicsCreated& Delegate)
{
	return OnSkelMeshPhysicsCreated.Add(Delegate);
}

void USkeletalMeshComponent::UnregisterOnPhysicsCreatedDelegate(const FDelegateHandle& DelegateHandle)
{
	OnSkelMeshPhysicsCreated.Remove(DelegateHandle);
}

FDelegateHandle USkeletalMeshComponent::RegisterOnTeleportDelegate(const FOnSkelMeshTeleported& Delegate)
{
	return OnSkelMeshPhysicsTeleported.Add(Delegate);
}

void USkeletalMeshComponent::UnregisterOnTeleportDelegate(const FDelegateHandle& DelegateHandle)
{
	OnSkelMeshPhysicsTeleported.Remove(DelegateHandle);
}

FDelegateHandle USkeletalMeshComponent::RegisterOnBoneTransformsFinalizedDelegate(const FOnBoneTransformsFinalizedMultiCast::FDelegate& Delegate)
{
	return OnBoneTransformsFinalizedMC.Add(Delegate);
}

void USkeletalMeshComponent::UnregisterOnBoneTransformsFinalizedDelegate(const FDelegateHandle& DelegateHandle)
{
	OnBoneTransformsFinalizedMC.Remove(DelegateHandle);
}

FDelegateHandle USkeletalMeshComponent::RegisterOnLODRequiredBonesUpdate(const FOnLODRequiredBonesUpdate& Delegate)
{
	return OnLODRequiredBonesUpdate_Static.Add(Delegate);
}

void USkeletalMeshComponent::UnregisterOnLODRequiredBonesUpdate(const FDelegateHandle& DelegateHandle)
{
	OnLODRequiredBonesUpdate_Static.Remove(DelegateHandle);
}

FDelegateHandle USkeletalMeshComponent::RegisterOnLODRequiredBonesUpdate_Member(const FOnLODRequiredBonesUpdate& Delegate)
{
	return OnLODRequiredBonesUpdate_Member.Add(Delegate);
}

void USkeletalMeshComponent::UnregisterOnLODRequiredBonesUpdate_Member(const FDelegateHandle& DelegateHandle)
{
	OnLODRequiredBonesUpdate_Member.Remove(DelegateHandle);
}

bool USkeletalMeshComponent::MoveComponentImpl(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* OutHit /*= nullptr*/, EMoveComponentFlags MoveFlags /*= MOVECOMP_NoFlags*/, ETeleportType Teleport /*= ETeleportType::None*/)
{
#if WITH_EDITOR
	UWorld* World = GetWorld();
	if(World && World->IsGameWorld())
	{
		if (FBodyInstance* BI = GetBodyInstance())
		{
			//If the root body is simulating and we're told to move without teleportation we warn. This is hard to support because of bodies chained together which creates some ambiguity
			if (BI->IsInstanceSimulatingPhysics() && Teleport == ETeleportType::None && (MoveFlags&EMoveComponentFlags::MOVECOMP_SkipPhysicsMove) == 0)
			{
				FMessageLog("PIE").Warning(FText::Format(LOCTEXT("MovingSimulatedSkeletalMesh", "Attempting to move a fully simulated skeletal mesh {0}. Please use the Teleport flag"),
					FText::FromString(GetPathNameSafe(this))));
			}
		}
	}
#endif

	bool bSuccess = Super::MoveComponentImpl(Delta, NewRotation, bSweep, OutHit, MoveFlags, Teleport);
	if(bSuccess && Teleport != ETeleportType::None)
	{
		// If a skeletal mesh component receives a teleport we should reset any other dynamic simulations
		ResetAnimInstanceDynamics(Teleport);

		OnSkelMeshPhysicsTeleported.Broadcast();
	}

	return bSuccess;
}

void USkeletalMeshComponent::AddFollowerPoseComponent(USkinnedMeshComponent* SkinnedMeshComponent)
{
	Super::AddFollowerPoseComponent(SkinnedMeshComponent);

	if(USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SkinnedMeshComponent))
	{
		SkeletalMeshComponent->bRequiredBonesUpToDate = false;
	}

	bRequiredBonesUpToDate = false;
}

void USkeletalMeshComponent::RemoveFollowerPoseComponent(USkinnedMeshComponent* SkinnedMeshComponent)
{
	Super::RemoveFollowerPoseComponent(SkinnedMeshComponent);

	if(USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SkinnedMeshComponent))
	{
		SkeletalMeshComponent->bRequiredBonesUpToDate = false;
	}

	bRequiredBonesUpToDate = false;
}

void USkeletalMeshComponent::SnapshotPose(FPoseSnapshot& Snapshot)
{
	USkeletalMesh* SkelMesh = GetSkeletalMeshAsset();
	if (ensureAsRuntimeWarning(SkelMesh != nullptr))
	{
		const TArray<FTransform>& ComponentSpaceTMs = GetComponentSpaceTransforms();
		const FReferenceSkeleton& RefSkeleton = SkelMesh->GetRefSkeleton();
		const TArray<FTransform>& RefPoseSpaceBaseTMs = RefSkeleton.GetRefBonePose();

		Snapshot.SkeletalMeshName = SkelMesh->GetFName();

		const int32 NumSpaceBases = ComponentSpaceTMs.Num();
		Snapshot.LocalTransforms.Reset(NumSpaceBases);
		Snapshot.LocalTransforms.AddUninitialized(NumSpaceBases);
		Snapshot.BoneNames.Reset(NumSpaceBases);
		Snapshot.BoneNames.AddUninitialized(NumSpaceBases);

		//Set root bone which is always evaluated.
		Snapshot.LocalTransforms[0] = ComponentSpaceTMs[0];
		Snapshot.BoneNames[0] = RefSkeleton.GetBoneName(0);

		int32 CurrentRequiredBone = 1;
		for (int32 ComponentSpaceIdx = 1; ComponentSpaceIdx < NumSpaceBases; ++ComponentSpaceIdx)
		{
			Snapshot.BoneNames[ComponentSpaceIdx] = RefSkeleton.GetBoneName(ComponentSpaceIdx);

			const bool bBoneHasEvaluated = FillComponentSpaceTransformsRequiredBones.IsValidIndex(CurrentRequiredBone) && ComponentSpaceIdx == FillComponentSpaceTransformsRequiredBones[CurrentRequiredBone];
			const int32 ParentIndex = RefSkeleton.GetParentIndex(ComponentSpaceIdx);
			ensureMsgf(ParentIndex != INDEX_NONE, TEXT("Getting an invalid parent bone for bone %d, but this should not be possible since this is not the root bone!"), ComponentSpaceIdx);

			const FTransform& ParentTransform = ComponentSpaceTMs[ParentIndex];
			const FTransform& ChildTransform = ComponentSpaceTMs[ComponentSpaceIdx];
			Snapshot.LocalTransforms[ComponentSpaceIdx] = bBoneHasEvaluated ? ChildTransform.GetRelativeTransform(ParentTransform) : RefPoseSpaceBaseTMs[ComponentSpaceIdx];

			if (bBoneHasEvaluated)
			{
				CurrentRequiredBone++;
			}
		}

		Snapshot.bIsValid = true;
	}
	else
	{
		Snapshot.bIsValid = false;
	}
}

void USkeletalMeshComponent::SetUpdateAnimationInEditor(const bool NewUpdateState)
{
	#if WITH_EDITOR
	if (IsRegistered())
	{
		bUpdateAnimationInEditor = NewUpdateState;
	}
	#endif
}

void USkeletalMeshComponent::SetUpdateClothInEditor(const bool NewUpdateState)
{
#if WITH_EDITOR
	if (IsRegistered())
	{
		bUpdateClothInEditor = NewUpdateState; 
	}
#endif
}

float USkeletalMeshComponent::GetTeleportRotationThreshold() const
{
	return TeleportRotationThreshold;
}

void USkeletalMeshComponent::SetTeleportRotationThreshold(float Threshold)
{
	TeleportRotationThreshold = Threshold;
	ClothTeleportCosineThresholdInRad = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportCosineRotationThreshold(TeleportRotationThreshold);
}

float USkeletalMeshComponent::GetTeleportDistanceThreshold() const
{
	return TeleportDistanceThreshold;
}

void USkeletalMeshComponent::SetTeleportDistanceThreshold(float Threshold)
{
	TeleportDistanceThreshold = Threshold;
	ClothTeleportDistThresholdSquared = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportDistanceThresholdSquared(TeleportDistanceThreshold);
}

void USkeletalMeshComponent::SetDisableAnimCurves(bool bInDisableAnimCurves)
{
	SetAllowAnimCurveEvaluation(!bInDisableAnimCurves);
}

void USkeletalMeshComponent::SetAllowAnimCurveEvaluation(bool bInAllow)
{
	if (bAllowAnimCurveEvaluation != bInAllow)
	{
		bAllowAnimCurveEvaluation = bInAllow;
		// clear cache uid version, so it will update required curves
		CachedAnimCurveUidVersion = 0;
	}
}

void USkeletalMeshComponent::AllowAnimCurveEvaluation(FName NameOfCurve, bool bAllow)
{
	if (bAllow == bFilteredAnimCurvesIsAllowList)
	{
		FilteredAnimCurves.Add(NameOfCurve);
		CachedAnimCurveUidVersion = 0;
	}
}

void USkeletalMeshComponent::ResetAllowedAnimCurveEvaluation()
{
	FilteredAnimCurves.Reset();
	CachedAnimCurveUidVersion = 0;
}

void USkeletalMeshComponent::SetAllowedAnimCurvesEvaluation(const TArray<FName>& List, bool bAllow)
{
	// Reset already clears the version - CachedAnimCurveUidVersion = 0;
	ResetAllowedAnimCurveEvaluation();
	FilteredAnimCurves.Append(List);
	bFilteredAnimCurvesIsAllowList = bAllow;
}

UE::Anim::FCurveFilterSettings USkeletalMeshComponent::GetCurveFilterSettings(int32 InLODOverride) const
{
	UE::Anim::FCurveFilterSettings CurveFilterSettings;
	if(bAllowAnimCurveEvaluation)
	{
		if(FilteredAnimCurves.Num() > 0)
		{
			CurveFilterSettings.FilterMode = bFilteredAnimCurvesIsAllowList ? UE::Anim::ECurveFilterMode::AllowOnlyFiltered : UE::Anim::ECurveFilterMode::DisallowFiltered;
			CurveFilterSettings.FilterCurves = &FilteredAnimCurves;
		}
		else
		{
			CurveFilterSettings.FilterMode = UE::Anim::ECurveFilterMode::DisallowFiltered;	// This applies LOD-based and bone-linked filtering
		}
	}
	else
	{
		CurveFilterSettings.FilterMode = UE::Anim::ECurveFilterMode::DisallowAll;
	}

	if(InLODOverride != INDEX_NONE)
	{
		CurveFilterSettings.LODIndex = InLODOverride;
	}
	else
	{
		CurveFilterSettings.LODIndex = GetPredictedLODLevel();
	}

	return CurveFilterSettings;
}

const TArray<FTransform>& USkeletalMeshComponent::GetCachedComponentSpaceTransforms() const
{
	return CachedComponentSpaceTransforms;
}


bool USkeletalMeshComponent::GetFloatAttribute_Ref(const FName& BoneName, const FName& AttributeName, float& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<float, FFloatAnimationAttribute>(BoneName, AttributeName, OutValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetVectorAttribute_Ref(const FName& BoneName, const FName& AttributeName, FVector& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<FVector, FVectorAnimationAttribute>(BoneName, AttributeName, OutValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetQuaternionAttribute_Ref(const FName& BoneName, const FName& AttributeName, FQuat& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<FQuat, FQuaternionAnimationAttribute>(BoneName, AttributeName, OutValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetTransformAttribute_Ref(const FName& BoneName, const FName& AttributeName, FTransform& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	bool bResult = FindAttributeChecked<FTransform, FTransformAnimationAttribute>(BoneName, AttributeName, OutValue, OutValue, LookupType);
	return bResult;
}

bool USkeletalMeshComponent::GetIntegerAttribute_Ref(const FName& BoneName, const FName& AttributeName, int32& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<int32, FIntegerAnimationAttribute>(BoneName, AttributeName, OutValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetStringAttribute_Ref(const FName& BoneName, const FName& AttributeName, FString& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<FString, FStringAnimationAttribute>(BoneName, AttributeName, OutValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetFloatAttribute(const FName& BoneName, const FName& AttributeName, float DefaultValue, float& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<float, FFloatAnimationAttribute>(BoneName, AttributeName, DefaultValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetVectorAttribute(const FName& BoneName, const FName& AttributeName, FVector DefaultValue, FVector& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<FVector, FVectorAnimationAttribute>(BoneName, AttributeName, DefaultValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetQuaternionAttribute(const FName& BoneName, const FName& AttributeName, FQuat DefaultValue, FQuat& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<FQuat, FQuaternionAnimationAttribute>(BoneName, AttributeName, DefaultValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetTransformAttribute(const FName& BoneName, const FName& AttributeName, FTransform DefaultValue, FTransform& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<FTransform, FTransformAnimationAttribute>(BoneName, AttributeName, DefaultValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetIntegerAttribute(const FName& BoneName, const FName& AttributeName, int32 DefaultValue, int32& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<int32, FIntegerAnimationAttribute>(BoneName, AttributeName, DefaultValue, OutValue, LookupType);
}

bool USkeletalMeshComponent::GetStringAttribute(const FName& BoneName, const FName& AttributeName, FString DefaultValue, FString& OutValue, ECustomBoneAttributeLookup LookupType /*= ECustomBoneAttributeLookup::BoneOnly*/)
{
	return FindAttributeChecked<FString, FStringAnimationAttribute>(BoneName, AttributeName, DefaultValue, OutValue, LookupType);
}

template<typename DataType, typename CustomAttributeType>
bool USkeletalMeshComponent::FindAttributeChecked(const FName& BoneName, const FName& AttributeName, DataType DefaultValue, DataType& OutValue, ECustomBoneAttributeLookup LookupType)
{
	OutValue = DefaultValue;
	bool bFound = false;

	USkeletalMesh* SkelMesh = GetSkeletalMeshAsset();
	if (SkelMesh)
	{
		const UE::Anim::FMeshAttributeContainer& Attributes = GetCustomAttributes();
		const int32 BoneIndex = SkelMesh->GetRefSkeleton().FindBoneIndex(BoneName);

		const CustomAttributeType* AttributePtr = Attributes.Find<CustomAttributeType>(UE::Anim::FAttributeId(AttributeName, FCompactPoseBoneIndex(BoneIndex)));

		if (AttributePtr == nullptr && BoneIndex != INDEX_NONE && LookupType != ECustomBoneAttributeLookup::BoneOnly)
		{
			if (LookupType == ECustomBoneAttributeLookup::ImmediateParent)
			{
				const int32 ParentIndex = SkelMesh->GetRefSkeleton().GetParentIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					AttributePtr = Attributes.Find<CustomAttributeType>(UE::Anim::FAttributeId(AttributeName, FCompactPoseBoneIndex(ParentIndex)));
				}
			}
			else if (LookupType == ECustomBoneAttributeLookup::ParentHierarchy)
			{
				int32 SearchBoneIndex = BoneIndex;
				int32 ParentIndex = SkelMesh->GetRefSkeleton().GetParentIndex(SearchBoneIndex);

				while (ParentIndex != INDEX_NONE)
				{
					AttributePtr = Attributes.Find<CustomAttributeType>(UE::Anim::FAttributeId(AttributeName, FCompactPoseBoneIndex(ParentIndex)));
					if (AttributePtr)
					{
						break;
					}

					SearchBoneIndex = ParentIndex;
					ParentIndex = SkelMesh->GetRefSkeleton().GetParentIndex(SearchBoneIndex);
				}
			}
		}

		if (AttributePtr != nullptr)
		{
			OutValue = AttributePtr->Value;
			bFound = true;
		}
	}

	return bFound;
}

bool USkeletalMeshComponent::GetCurveValue(FName CurveName, float DefaultValue, float& Value) const
{
	bool bHasElement = false;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Value = AnimCurves.Get(CurveName, bHasElement, DefaultValue);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return bHasElement;
}

const FBlendedHeapCurve& USkeletalMeshComponent::GetAnimCurves() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return AnimCurves;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FName& USkeletalMeshComponent::GetAnimationModePropertyNameChecked()
{
	static FName Name = GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, AnimationMode);
	return Name;
}

#if WITH_EDITORONLY_DATA
FName USkeletalMeshComponent::GetSkeletalMeshAssetPropertyNameChecked()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, SkeletalMeshAsset);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITORONLY_DATA

TArray<FTransform> USkeletalMeshComponent::GetBoneSpaceTransforms() 
{
	// We may be doing parallel evaluation on the current anim instance
	// Calling this here with true will block until that thread completes
	// and it is safe to continue
	const bool bBlockOnTask = true; // wait on evaluation task so it is safe to swap the buffers
	const bool bPerformPostAnimEvaluation = true; // Do PostEvaluation so we make sure to swap the buffers back. 
	HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return BoneSpaceTransforms;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TArrayView<const FTransform> USkeletalMeshComponent::GetBoneSpaceTransformsView()
{
	// We may be doing parallel evaluation on the current anim instance
	// Calling this here with true will block until that thread completes
	// and it is safe to continue
	const bool bBlockOnTask = true; // wait on evaluation task so it is safe to swap the buffers
	const bool bPerformPostAnimEvaluation = true; // Do PostEvaluation so we make sure to swap the buffers back. 
	HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return BoneSpaceTransforms;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
void USkeletalMeshComponent::UpdatePoseWatches()
{
	PoseWatches.Empty();
	
	auto CopyPoseWatches = [this](const UAnimInstance* InAnimInstance)
	{
		if (InAnimInstance && InAnimInstance->IsBeingDebugged())
		{
			if (const UAnimBlueprintGeneratedClass* AnimBPGenClass = Cast<UAnimBlueprintGeneratedClass>(InAnimInstance->GetClass()))
			{
				if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(AnimBPGenClass->ClassGeneratedBy))
				{
					if (!GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, false))
					{
						return;
					}

					UAnimBlueprint* RootAnimBlueprint = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint);
					AnimBlueprint = RootAnimBlueprint ? RootAnimBlueprint : AnimBlueprint;

					const FAnimBlueprintDebugData& DebugData = AnimBlueprint->GetAnimBlueprintGeneratedClass()->GetAnimBlueprintDebugData();
					for (const FAnimNodePoseWatch& PoseWatch : DebugData.AnimNodePoseWatch)
					{
						if (const UPoseWatchPoseElement* PoseWatchPoseElement = PoseWatch.PoseWatchPoseElement)
						{
							if (PoseWatchPoseElement->GetIsEnabled() && PoseWatchPoseElement->GetIsVisible())
							{
								PoseWatches.Add(PoseWatch);
							}
						}
					}	
				}
			}
		}
	};
	
	ForEachAnimInstance(CopyPoseWatches);
}
#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE

