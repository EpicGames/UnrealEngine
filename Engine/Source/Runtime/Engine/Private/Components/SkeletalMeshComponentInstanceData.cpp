// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshComponentInstanceData.h"

#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshComponentInstanceData)


FSkeletalMeshComponentInstanceData::FSkeletalMeshComponentInstanceData(const USkeletalMeshComponent* SourceComponent)
	: FSceneComponentInstanceData(SourceComponent)
	, bUpdateAnimationInEditor(0)
	, bUpdateClothInEditor(0)

{
#if WITH_EDITOR
	// Only Blueprint components would reset tansient values when the construction script is re-run.
	// Hence, we only need to apply instance cache for Blueprint created components.
	const bool bIsBlueprintCreatedComponent = SourceComponent->CreationMethod == EComponentCreationMethod::SimpleConstructionScript
		|| SourceComponent->CreationMethod == EComponentCreationMethod::UserConstructionScript;
	if (bIsBlueprintCreatedComponent)
	{
		bUpdateAnimationInEditor = SourceComponent->GetUpdateAnimationInEditor();
		bUpdateClothInEditor = SourceComponent->GetUpdateClothInEditor();
	}
#endif // WITH_EDITOR
}

bool FSkeletalMeshComponentInstanceData::ContainsData() const
{
	return bUpdateAnimationInEditor || bUpdateClothInEditor;
}

void FSkeletalMeshComponentInstanceData::ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase)
{
	Super::ApplyToComponent(Component, CacheApplyPhase);

#if WITH_EDITOR
	if (USkeletalMeshComponent * SkeletalMesh = Cast<USkeletalMeshComponent>(Component))
	{
		SkeletalMesh->RefreshSkeletalMeshAsset();
		SkeletalMesh->SetUpdateAnimationInEditor(bUpdateAnimationInEditor);
		SkeletalMesh->SetUpdateClothInEditor(bUpdateClothInEditor);
	}
#endif // WITH_EDITOR
}
