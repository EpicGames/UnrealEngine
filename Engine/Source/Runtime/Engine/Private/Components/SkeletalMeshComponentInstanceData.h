// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/SceneComponent.h"

#include "SkeletalMeshComponentInstanceData.generated.h"

class USkeletalMeshComponent;

/** Saves internal SkeletalMesh (transient) state that gets lost at reconstruction for Blueprint created components. */
USTRUCT()
struct FSkeletalMeshComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()
	
	FSkeletalMeshComponentInstanceData() = default;
	explicit FSkeletalMeshComponentInstanceData(const USkeletalMeshComponent* SourceComponent);
	
	virtual bool ContainsData() const override;
	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override;

	uint8 bUpdateAnimationInEditor : 1;
	uint8 bUpdateClothInEditor : 1;
};
