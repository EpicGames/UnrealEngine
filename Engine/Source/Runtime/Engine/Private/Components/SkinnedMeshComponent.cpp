// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnSkeletalComponent.cpp: Actor component implementation.
=============================================================================*/

#include "Components/SkinnedMeshComponent.h"
#include "Engine/SkinnedAsset.h"
#include "HAL/LowLevelMemStats.h"
#include "RenderingThread.h"
#include "GameFramework/PlayerController.h"
#include "ContentStreaming.h"
#include "Materials/MaterialInterface.h"
#include "UnrealEngine.h"
#include "SceneInterface.h"
#include "SkeletalRenderCPUSkin.h"
#include "SkeletalRenderGPUSkin.h"
#include "SkeletalRenderNanite.h"
#include "SkeletalRenderStatic.h"
#include "Animation/AnimStats.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "Engine/MaterialOverlayHelper.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetCommon.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Engine/CollisionProfile.h"
#include "SkeletalMeshSceneProxy.h"
#include "Animation/MeshDeformer.h"
#include "Animation/MeshDeformerInstance.h"
#include "Animation/MeshDeformerProvider.h"
#include "AnimationRuntime.h"
#include "BoneWeights.h"
#include "Animation/SkinWeightProfileManager.h"
#include "GPUSkinCache.h"
#include "PSOPrecache.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "SkeletalRender.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "UObject/Package.h"
#include "Rendering/RenderCommandPipes.h"
#include "Rendering/NaniteResources.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "StateStream/SkinnedMeshStateStream.h"
#include "TransformStateStream.h"
#include "ComponentRecreateRenderStateContext.h"
#include "NaniteSceneProxy.h"
#include "NaniteVertexFactory.h"
#include "ObjectCacheEventSink.h"
#include "Animation/MeshDeformerGeometryReadback.h"
#include "Misc/DataValidation.h"
#include "Rendering/SkeletalMeshHalfEdgeBufferAccessor.h"
#include "SkinnedMeshComponentHelper.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/UObjectMigrationContext.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkinnedMeshComponent)

DEFINE_LOG_CATEGORY_STATIC(LogSkinnedMeshComp, Log, All);
#define LOCTEXT_NAMESPACE "SkinnedMeshComponent"

namespace UE::Anim
{
	bool bUseSafeMeshPoseIndices = false;
	FAutoConsoleVariableRef CVarUseSafeMeshPoseIndices(TEXT("a.UseSafeMeshPoseIndices"), bUseSafeMeshPoseIndices
		, TEXT("Whether to compute and use valid mesh pose indices on skeletal mesh components. This allows us to safely reset missing bones in leader/follower setups. This has a small additional cost.")
		, ECVF_Default);
}

#if !UE_BUILD_SHIPPING
int32 GSkeletalMeshForceLOD = -1;
FAutoConsoleVariableRef CVarSkeletalMeshForceLOD(
	TEXT("r.SkeletalMeshForceLOD"),
	GSkeletalMeshForceLOD,
	TEXT("Forces LOD"),
	ECVF_Default
	);
#endif

int32 GSkeletalMeshLODBias = 0;
FAutoConsoleVariableRef CVarSkeletalMeshLODBias(
	TEXT("r.SkeletalMeshLODBias"),
	GSkeletalMeshLODBias,
	TEXT("LOD bias for skeletal meshes (does not affect animation editor viewports)."),
	ECVF_Scalability
	);

static TAutoConsoleVariable<int32> CVarEnableAnimRateOptimization(
	TEXT("a.URO.Enable"),
	1,
	TEXT("True to anim rate optimization."));

static TAutoConsoleVariable<int32> CVarDrawAnimRateOptimization(
	TEXT("a.URO.Draw"),
	0,
	TEXT("True to draw color coded boxes for anim rate."));

static TAutoConsoleVariable<int32> CVarEnableMorphTargets(TEXT("r.EnableMorphTargets"), 1, TEXT("Enable Morph Targets"));

static TAutoConsoleVariable<int32> CVarAnimVisualizeLODs(
	TEXT("a.VisualizeLODs"),
	0,
	TEXT("Visualize SkelMesh LODs"));

float GUpdateBoundsNotifyStreamingRadiusChangeRatio = 0.1f;
FAutoConsoleVariableRef CVarUpdateBoundsNotifyStreamingRadiusChangeRatio(
	TEXT("r.SkinnedMesh.UpdateBoundsNotifyStreamingRadiusChangeRatio"),
	GUpdateBoundsNotifyStreamingRadiusChangeRatio,
	TEXT("Update the streaming manager when the radius changes by more than this ratio since the last update. A negative value will disable the update."),
	ECVF_Default
);

static bool GReleasePreviousLODInfoOnInitialization = true;
static FAutoConsoleVariableRef CVarReleasePreviousLODInfoOnInitialization(
	TEXT("r.SkinnedMesh.ReleasePreviousLODInfoOnInitialization"),
	GReleasePreviousLODInfoOnInitialization,
	TEXT("Whether to flush the render thread (incurring a game thread stall) and clean existing LOD info when re-initializating."),
	ECVF_Default
);

int32 GSkinnedMeshRenderNanite = 1;
static FAutoConsoleVariableRef CVarSkinnedMeshRenderNanite(
	TEXT("r.SkinnedMesh.RenderNanite"),
	GSkinnedMeshRenderNanite,
	TEXT("When true, allows skinned meshes to render as Nanite when enabled on the mesh (otherwise uses fallback mesh)."),
	FConsoleVariableDelegate::CreateLambda(
		[](IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
		}
	)
);

static TAutoConsoleVariable<int32> CVarSkinnedMeshSyncStreamingLODClamp(
	TEXT("r.SkinnedMesh.SyncStreamingLODClamp"),
	1,
	TEXT("Whether to sync streaming LOD index clamp among components with SyncAttachParentLOD enabled in the attachment tree."),
	ECVF_Default);


bool USkinnedMeshComponent::ShouldRenderNaniteSkinnedMeshes()
{
	return NaniteSkinnedMeshesSupported() && GSkinnedMeshRenderNanite != 0;
}

namespace FAnimUpdateRateManager
{
	static float TargetFrameTimeForUpdateRate = 1.f / 30.f; //Target frame rate for lookahead URO

	// Bucketed group counters to stagged update an eval, used to initialise AnimUpdateRateShiftTag
	// for mesh params in the same shift group.
	struct FShiftBucketParameters
	{
		void SetFriendlyName(const EUpdateRateShiftBucket& InShiftBucket, const FName& InFriendlyName)
		{
			ShiftTagFriendlyNames[(uint8)InShiftBucket] = InFriendlyName;
		}

		const FName& GetFriendlyName(const EUpdateRateShiftBucket& InShiftBucket)
		{
			return ShiftTagFriendlyNames[(uint8)InShiftBucket];
		}

		static uint8 ShiftTagBuckets[(uint8)EUpdateRateShiftBucket::ShiftBucketMax];

	private:
		static FName ShiftTagFriendlyNames[(uint8)EUpdateRateShiftBucket::ShiftBucketMax];
	};
	uint8 FShiftBucketParameters::ShiftTagBuckets[] = {};
	FName FShiftBucketParameters::ShiftTagFriendlyNames[] = {};

	struct FAnimUpdateRateParametersTracker
	{
		FAnimUpdateRateParameters UpdateRateParameters;

		/** Frame counter to call AnimUpdateRateTick() just once per frame. */
		uint32 AnimUpdateRateFrameCount;

		/** Counter to stagger update and evaluation across skinned mesh components */
		uint8 AnimUpdateRateShiftTag;

		/** List of all USkinnedMeshComponents that use this set of parameters */
		TArray<USkinnedMeshComponent*> RegisteredComponents;

		FAnimUpdateRateParametersTracker() : AnimUpdateRateFrameCount(0), AnimUpdateRateShiftTag(0) {}

		uint8 GetAnimUpdateRateShiftTag(const EUpdateRateShiftBucket& ShiftBucket)
		{
			// If hasn't been initialized yet, pick a unique ID, to spread population over frames.
			if (AnimUpdateRateShiftTag == 0)
			{
				AnimUpdateRateShiftTag = ++FShiftBucketParameters::ShiftTagBuckets[(uint8)ShiftBucket];
			}

			return AnimUpdateRateShiftTag;
		}

		bool IsHumanControlled() const
		{
			AActor* Owner = RegisteredComponents[0]->GetOwner();
			APlayerController* Controller = Owner ? Owner->GetInstigatorController<APlayerController>() : nullptr;
			return Controller != nullptr;
		}
	};

	TMap<UObject*, FAnimUpdateRateParametersTracker*> ActorToUpdateRateParams;

	UObject* GetMapIndexForComponent(USkinnedMeshComponent* SkinnedComponent)
	{
		UObject* TrackerIndex = SkinnedComponent->GetOwner();
		if (TrackerIndex == nullptr)
		{
			TrackerIndex = SkinnedComponent;
		}
		return TrackerIndex;
	}

	FAnimUpdateRateParameters* GetUpdateRateParameters(USkinnedMeshComponent* SkinnedComponent)
	{
		if (!SkinnedComponent)
		{
			return nullptr;
		}
		UObject* TrackerIndex = GetMapIndexForComponent(SkinnedComponent);

		FAnimUpdateRateParametersTracker** ExistingTrackerPtr = ActorToUpdateRateParams.Find(TrackerIndex);
		if (!ExistingTrackerPtr)
		{
			ExistingTrackerPtr = &ActorToUpdateRateParams.Add(TrackerIndex);
			(*ExistingTrackerPtr) = new FAnimUpdateRateParametersTracker();
		}
		
		check(ExistingTrackerPtr);
		FAnimUpdateRateParametersTracker* ExistingTracker = *ExistingTrackerPtr;
		check(ExistingTracker);
		checkSlow(!ExistingTracker->RegisteredComponents.Contains(SkinnedComponent)); // We have already been registered? Something has gone very wrong!

		ExistingTracker->RegisteredComponents.Add(SkinnedComponent);
		FAnimUpdateRateParameters* UpdateRateParams = &ExistingTracker->UpdateRateParameters;
		SkinnedComponent->OnAnimUpdateRateParamsCreated.ExecuteIfBound(UpdateRateParams);

		return UpdateRateParams;
	}

	void CleanupUpdateRateParametersRef(USkinnedMeshComponent* SkinnedComponent)
	{
		const UObject* TrackerIndex = GetMapIndexForComponent(SkinnedComponent);
		FAnimUpdateRateParametersTracker** TrackerPtr = ActorToUpdateRateParams.Find(TrackerIndex);
		if (TrackerPtr)
		{
			FAnimUpdateRateParametersTracker* Tracker = *TrackerPtr;
			Tracker->RegisteredComponents.Remove(SkinnedComponent);
			if (Tracker->RegisteredComponents.Num() == 0)
			{
				ActorToUpdateRateParams.Remove(TrackerIndex);
				delete Tracker;
			}
		}
	}

	static TAutoConsoleVariable<int32> CVarForceAnimRate(
		TEXT("a.URO.ForceAnimRate"),
		0,
		TEXT("Non-zero to force anim rate. 10 = eval anim every ten frames for those meshes that can do it. In some cases a frame is considered to be 30fps."));

	static TAutoConsoleVariable<int32> CVarForceInterpolation(
		TEXT("a.URO.ForceInterpolation"),
		0,
		TEXT("Set to 1 to force interpolation"));

	static TAutoConsoleVariable<int32> CVarURODisableInterpolation(
		TEXT("a.URO.DisableInterpolation"),
		0,
		TEXT("Set to 1 to disable interpolation"));

	void AnimUpdateRateSetParams(FAnimUpdateRateParametersTracker* Tracker, float DeltaTime, bool bRecentlyRendered, float MaxDistanceFactor, int32 MinLod, bool bNeedsValidRootMotion, bool bUsingRootMotionFromEverything)
	{
		// default rules for setting update rates

		// Human controlled characters should be ticked always fully to minimize latency w/ game play events triggered by animation.
		const bool bHumanControlled = Tracker->IsHumanControlled();

		bool bNeedsEveryFrame = bNeedsValidRootMotion && !bUsingRootMotionFromEverything;

		// Not rendered, including dedicated servers. we can skip the Evaluation part.
		if (!bRecentlyRendered)
		{
			const int32 NewUpdateRate = ((bHumanControlled || bNeedsEveryFrame) ? 1 : Tracker->UpdateRateParameters.BaseNonRenderedUpdateRate);
			const int32 NewEvaluationRate = Tracker->UpdateRateParameters.BaseNonRenderedUpdateRate;
			Tracker->UpdateRateParameters.SetTrailMode(DeltaTime, Tracker->GetAnimUpdateRateShiftTag(Tracker->UpdateRateParameters.ShiftBucket), NewUpdateRate, NewEvaluationRate, false);
		}
		// Visible controlled characters or playing root motion. Need evaluation and ticking done every frame.
		else  if (bHumanControlled || bNeedsEveryFrame)
		{
			Tracker->UpdateRateParameters.SetTrailMode(DeltaTime, Tracker->GetAnimUpdateRateShiftTag(Tracker->UpdateRateParameters.ShiftBucket), 1, 1, false);
		}
		else
		{
			int32 DesiredEvaluationRate = 1;

			if(!Tracker->UpdateRateParameters.bShouldUseLodMap)
			{
				DesiredEvaluationRate = Tracker->UpdateRateParameters.BaseVisibleDistanceFactorThesholds.Num() + 1;
				for(int32 Index = 0; Index < Tracker->UpdateRateParameters.BaseVisibleDistanceFactorThesholds.Num(); Index++)
				{
					const float& DistanceFactorThreadhold = Tracker->UpdateRateParameters.BaseVisibleDistanceFactorThesholds[Index];
					if(MaxDistanceFactor > DistanceFactorThreadhold)
					{
						DesiredEvaluationRate = Index + 1;
						break;
					}
				}
			}
			else
			{
				// Using LOD map which should have been set along with flag in custom delegate on creation.
				// if the map is empty don't throttle
				if(int32* FrameSkip = Tracker->UpdateRateParameters.LODToFrameSkipMap.Find(MinLod))
				{
					// Add 1 as an eval rate of 1 is 0 frameskip
					DesiredEvaluationRate = (*FrameSkip) + 1;
				}
				// We haven't found our LOD number into our array. :(
				// Default to matching settings of previous highest LOD number we've found.
				// For example if we're missing LOD 3, and we have settings for LOD 2, then match that.
				// Having no settings means we default to evaluating every frame, which is highest quality setting we have.
				// This is not what we want to higher LOD numbers.
				else if (Tracker->UpdateRateParameters.LODToFrameSkipMap.Num() > 0)
				{
					TMap<int32, int32>& LODToFrameSkipMap = Tracker->UpdateRateParameters.LODToFrameSkipMap;
					for (auto Iter = LODToFrameSkipMap.CreateConstIterator(); Iter; ++Iter)
					{
						if (Iter.Key() < MinLod)
						{
							DesiredEvaluationRate = FMath::Max(Iter.Value(), DesiredEvaluationRate);
						}
					}

					// Cache result back into TMap, so we don't have to do this every frame.
					LODToFrameSkipMap.Add(MinLod, DesiredEvaluationRate);

					// Add 1 as an eval rate of 1 is 0 frameskip
					DesiredEvaluationRate++;
				}
			}

			int32 ForceAnimRate = CVarForceAnimRate.GetValueOnGameThread();
			if (ForceAnimRate)
			{
				DesiredEvaluationRate = ForceAnimRate;
			}

			if (bUsingRootMotionFromEverything && DesiredEvaluationRate > 1)
			{
				//Use look ahead mode that allows us to rate limit updates even when using root motion
				Tracker->UpdateRateParameters.SetLookAheadMode(DeltaTime, Tracker->GetAnimUpdateRateShiftTag(Tracker->UpdateRateParameters.ShiftBucket), TargetFrameTimeForUpdateRate*DesiredEvaluationRate);
			}
			else
			{
				Tracker->UpdateRateParameters.SetTrailMode(DeltaTime, Tracker->GetAnimUpdateRateShiftTag(Tracker->UpdateRateParameters.ShiftBucket), DesiredEvaluationRate, DesiredEvaluationRate, true);
			}
		}
	}

	void AnimUpdateRateTick(FAnimUpdateRateParametersTracker* Tracker, float DeltaTime, bool bNeedsValidRootMotion)
	{
		// Go through components and figure out if they've been recently rendered, and the biggest MaxDistanceFactor
		bool bRecentlyRendered = false;
		bool bPlayingNetworkedRootMotionMontage = false;
		bool bUsingRootMotionFromEverything = true;
		float MaxDistanceFactor = 0.f;
		int32 MinLod = MAX_int32;

		const TArray<USkinnedMeshComponent*>& SkinnedComponents = Tracker->RegisteredComponents;
		for (USkinnedMeshComponent* Component : SkinnedComponents)
		{
			if (Component->IsVisible())
			{
				bRecentlyRendered |= Component->bRecentlyRendered;
				MaxDistanceFactor = FMath::Max(MaxDistanceFactor, Component->MaxDistanceFactor);
				bPlayingNetworkedRootMotionMontage |= Component->IsPlayingNetworkedRootMotionMontage();
				bUsingRootMotionFromEverything &= Component->IsPlayingRootMotionFromEverything();
				MinLod = FMath::Min(MinLod, Tracker->UpdateRateParameters.bShouldUseMinLod ? Component->MinLodModel : Component->GetPredictedLODLevel());
			}
		}

		bNeedsValidRootMotion &= bPlayingNetworkedRootMotionMontage;

		// Figure out which update rate should be used.
		AnimUpdateRateSetParams(Tracker, DeltaTime, bRecentlyRendered, MaxDistanceFactor, MinLod, bNeedsValidRootMotion, bUsingRootMotionFromEverything);
	}

	const TCHAR* B(bool b)
	{
		return b ? TEXT("true") : TEXT("false");
	}

	void TickUpdateRateParameters(USkinnedMeshComponent* SkinnedComponent, float DeltaTime, bool bNeedsValidRootMotion)
	{
		// Convert current frame counter from 64 to 32 bits.
		const uint32 CurrentFrame32 = uint32(GFrameCounter % MAX_uint32);

		UObject* TrackerIndex = GetMapIndexForComponent(SkinnedComponent);
		FAnimUpdateRateParametersTracker* Tracker = ActorToUpdateRateParams.FindChecked(TrackerIndex);

		if (CurrentFrame32 != Tracker->AnimUpdateRateFrameCount)
		{
			Tracker->AnimUpdateRateFrameCount = CurrentFrame32;
			AnimUpdateRateTick(Tracker, DeltaTime, bNeedsValidRootMotion);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

void FExternalMorphSetWeights::UpdateNumActiveMorphTargets()
{
	NumActiveMorphTargets = 0;
	for (int32 Index = 0; Index < Weights.Num(); ++Index)
	{
		if (FMath::Abs<float>(Weights[Index]) >= ActiveWeightThreshold)
		{
			NumActiveMorphTargets++;
		}
	}
}

void FExternalMorphSetWeights::ZeroWeights(bool bZeroNumActiveMorphTargets)
{
	const int32 NumWeights = Weights.Num();
	for (int32 Index = 0; Index < NumWeights; ++Index)
	{
		Weights[Index] = 0.0f;
	}

	if (bZeroNumActiveMorphTargets)
	{
		NumActiveMorphTargets = 0;
	}
}

//////////////////////////////////////////////////////////////////////////

void FExternalMorphWeightData::UpdateNumActiveMorphTargets()
{
	NumActiveMorphTargets = 0;
	for (auto& MapItem : MorphSets)
	{
		MapItem.Value.UpdateNumActiveMorphTargets();
		NumActiveMorphTargets += MapItem.Value.NumActiveMorphTargets;
	}
}

//////////////////////////////////////////////////////////////////////////

USkinnedMeshComponent::USkinnedMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MeshObjectFactory(nullptr)
	, MeshObjectFactoryUserData(nullptr)
	, PreviousMeshObject(nullptr)
	, AnimUpdateRateParams(nullptr)
{
	bAutoActivate = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;

	VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	StreamingDistanceMultiplier = 1.0f;
	bCanHighlightSelectedSections = false;
	CanCharacterStepUpOn = ECB_Owner;
#if WITH_EDITORONLY_DATA
	SectionIndexPreview = -1;
	MaterialIndexPreview = -1;

	SelectedEditorSection = INDEX_NONE;
	SelectedEditorMaterial = INDEX_NONE;
#endif // WITH_EDITORONLY_DATA
	bPerBoneMotionBlur = true;
	bCastCapsuleDirectShadow = false;
	bCastCapsuleIndirectShadow = false;
	CapsuleIndirectShadowMinVisibility = .1f;

	bEvaluateWorldPositionOffset = true;
	bDisallowNanite = false;
	bForceDisableNanite = false;

	bDoubleBufferedComponentSpaceTransforms = true;
	LastStreamerUpdateBoundsRadius = -1.0;
	CurrentEditableComponentTransforms = 0;
	CurrentReadComponentTransforms = 1;
	bNeedToFlipSpaceBaseBuffers = false;
	bValidMeshPoseTransformsUpToDate = false;
	bBoneVisibilityDirty = false;

	bCanEverAffectNavigation = false;
	LeaderBoneMapCacheCount = 0;
	bSyncAttachParentLOD = true;
	bIgnoreLeaderPoseComponentLOD = false;

	PreviousBoneTransformRevisionNumber = 0;
	CurrentBoneTransformRevisionNumber = 0;
	CurrentBoneTransformFrame = 0;

	ExternalInterpolationAlpha = 0.0f;
	ExternalDeltaTime = 0.0f;
	ExternalTickRate = 1;
	bExternalInterpolate = false;
	bExternalUpdate = false;
	bExternalEvaluationRateLimited = false;
	bExternalTickRateControlled = false;

	MaxFirstLODIdxInAttachmentTree = 0;
	bMipLevelCallbackRegistered = false;

	bFollowerShouldTickPose = false;

	bHasSocketAttachments = false;

#if UE_ENABLE_DEBUG_DRAWING
	bDrawDebugSkeleton = false;
#endif

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PredictedLODLevel = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

USkinnedMeshComponent::USkinnedMeshComponent(FVTableHelper& Helper) : Super(Helper) {}
USkinnedMeshComponent::~USkinnedMeshComponent() = default;

void USkinnedMeshComponent::UpdateMorphMaterialUsageOnProxy()
{
	if (!SceneProxy)
	{
		return;
	}
	TArray<UMaterialInterface*> MaterialUsingMorphTarget;

	// Collect all sections on the underlying mesh that are going to be modified by morph targets, 
	if (!ActiveMorphTargets.IsEmpty())
	{
		TSet<int32> UsedMaterialIndices;

		const TIndirectArray<FSkeletalMeshLODRenderData>& LODRenderData = GetSkinnedAsset()->GetResourceForRendering()->LODRenderData;
		for (const TTuple<const UMorphTarget*, int32> ActiveMorphTargetInfo: ActiveMorphTargets)
		{
			const UMorphTarget* MorphTarget = ActiveMorphTargetInfo.Key;
			if (!MorphTarget)
			{
				continue;
			}

			const TArray<FMorphTargetLODModel>& MorphLODModels = MorphTarget->GetMorphLODModels();
			for (int32 LODIndex = 0; LODIndex < FMath::Min(LODRenderData.Num(), MorphLODModels.Num()); LODIndex++)
			{
				const FSkeletalMeshLODRenderData& RenderData = LODRenderData[LODIndex];
				
				for(const int32 MorphSectionIndex: MorphLODModels[LODIndex].SectionIndices)
				{
					if (RenderData.RenderSections.IsValidIndex(MorphSectionIndex))
					{
						UsedMaterialIndices.Add(RenderData.RenderSections[MorphSectionIndex].MaterialIndex);
					}
				}
			}
		}
		for (const int32 MaterialIndex: UsedMaterialIndices)
		{
			MaterialUsingMorphTarget.Add(GetMaterial(MaterialIndex));
		}
	}

	// If no morph targets are active, then this function needs to know that as well.
	if (SceneProxy->IsNaniteMesh())
	{
		// Nanite::FSkinnedSceneProxy
		// TODO: Nanite-Skinning
	}
	else
	{
		static_cast<FSkeletalMeshSceneProxy*>(SceneProxy)->UpdateMorphMaterialUsage_GameThread(MaterialUsingMorphTarget);
	}
}


void USkinnedMeshComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Get Mesh Object's memory
	if (MeshObject)
	{
		MeshObject->GetResourceSizeEx(CumulativeResourceSize);
	}
}

FPrimitiveSceneProxy* USkinnedMeshComponent::CreateSceneProxy()
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	ERHIFeatureLevel::Type SceneFeatureLevel = GetWorld()->GetFeatureLevel();
	FPrimitiveSceneProxy* Result = nullptr;
	FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();

	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogSkinnedMeshComp, Verbose, "Skipping CreateSceneProxy for USkinnedMeshComponent %ls (USkinnedMeshComponent PSOs are still compiling)", *GetFullName());
		return nullptr;
	}

	// Only create a scene proxy for rendering if properly initialized
	if (SkelMeshRenderData &&
		SkelMeshRenderData->LODRenderData.IsValidIndex(GetPredictedLODLevel()) &&
		!bHideSkin &&
		MeshObject)
	{
		if (MeshObject->IsNaniteMesh() && ShouldNaniteSkin())
		{
			check(MeshObject->GetNaniteMaterials());
			Result = ::new Nanite::FSkinnedSceneProxy(*MeshObject->GetNaniteMaterials(), this, SkelMeshRenderData);
		}
		else
		{
			Result = ::new FSkeletalMeshSceneProxy(this, SkelMeshRenderData);
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	SendRenderDebugPhysics(Result);
#endif

	return Result;
}

void USkinnedMeshComponent::AllowDuplicateTransformUpdates(bool bAllow)
{
	if (SceneProxy)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SceneProxy->SetCanSkipRedundantTransformUpdates(!bAllow);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

// UObject interface
// Override to have counting working better
void USkinnedMeshComponent::Serialize(FArchive& Ar)
{
	NotifyIfSkinnedAssetChanged();

	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		// add all native variables - mostly bigger chunks 
		ComponentSpaceTransformsArray[0].CountBytes(Ar);
		ComponentSpaceTransformsArray[1].CountBytes(Ar);
		LeaderBoneMap.CountBytes(Ar);
	}

	NotifyIfSkinnedAssetChanged();

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Ar.IsMigratingRemoteObjects())
	{
		// TODO: Optimize by re-initializing anims (AllocateTransformData + InitAnim in SkeletalMeshComponent::PostMigrate) instead of serializing.
		// Blocked by UE-363679 (server hang in PostMigrate).
		Ar << ComponentSpaceTransformsArray[0];
		Ar << ComponentSpaceTransformsArray[1];
	}

#endif

}

void USkinnedMeshComponent::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (SkeletalMesh)
	{
		SkinnedAsset = SkeletalMesh;  // Set to the SkeletalMesh pointer for backward compatibility when the SkinnedAsset is invalid
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	NotifyIfSkinnedAssetChanged();

	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AddSetMeshDeformerFlag)
	{
		// Set default for new bSetMeshDeformer flag.
		bSetMeshDeformer = MeshDeformer != nullptr;
	}

	PrecachePSOs();
}

void USkinnedMeshComponent::PostApplyToComponent()
{
	NotifyIfSkinnedAssetChanged();
	
	Super::PostApplyToComponent();
}

void USkinnedMeshComponent::PostReinitProperties()
{
	NotifyIfSkinnedAssetChanged();
	
	Super::PostReinitProperties();
}

void USkinnedMeshComponent::PostInitProperties()
{
	NotifyIfSkinnedAssetChanged();
	
	Super::PostInitProperties();
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
void USkinnedMeshComponent::PostMigrate(const struct FUObjectMigrationContext& MigrationContext)
{
	Super::PostMigrate(MigrationContext);

	if (MigrationContext.MigrationSide == EObjectMigrationSide::Send)
	{
		// When migrating-out, remove us from the local non-serialized cache
		if (AnimUpdateRateParams)
		{
			FAnimUpdateRateManager::CleanupUpdateRateParametersRef(this);
			AnimUpdateRateParams = nullptr;
		}
	}
	else if (ensure(MigrationContext.MigrationSide == EObjectMigrationSide::Receive))
	{
		// When migrating-in, reconstruct this and add it to the local non-serialized cache
		AnimUpdateRateParams = FAnimUpdateRateManager::GetUpdateRateParameters(this);
	}
}
#endif

void USkinnedMeshComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	if (GetSkinnedAsset() == nullptr ||
		GetSkinnedAsset()->GetResourceForRendering() == nullptr)
	{
		return;
	}

	FSkeletalMeshRenderData* RenderData = GetSkinnedAsset()->GetResourceForRendering();

	// Ensure LOD render data has been serialized/streamed in by now. If data is not ready, it is possible
	// the data is being loaded in from a different thread causing potential data races. 
	if (!RenderData->IsLODDataReady())
	{
		return;
	}

	// TODO: Nanite-Skinning

	ERHIFeatureLevel::Type FeatureLevel = GetWorld() ? GetWorld()->GetFeatureLevel() : GMaxRHIFeatureLevel;
	int32 MinLODIndex = ComputeMinLOD();
	bool bCPUSkin = bRenderStatic || ShouldCPUSkin();

	FPSOPrecacheVertexFactoryDataPerMaterialIndexList VFsPerMaterials;
	GetSkinnedAsset()->GetVertexFactoryTypesPerMaterialIndex(VFsPerMaterials, this, MinLODIndex, bCPUSkin, FeatureLevel);
	bool bAnySectionCastsShadows = RenderData->AnyRenderSectionCastsShadows(MinLODIndex);

	FPSOPrecacheParams PrecachePSOParams = BasePrecachePSOParams;
	PrecachePSOParams.bCastShadow = PrecachePSOParams.bCastShadow && bAnySectionCastsShadows;
	PrecachePSOParams.bSkinnedMesh = true;
	PrecachePSOParams.bRequireRTDynamicGeometryUpdate = PrecachePSOParams.bAnyMaterialHasWorldPositionOffset;

	// Skinned assets shouldn't need dynamic indirect lighting but MDCs for LumenCardCapture can still be setup and created (but not actually used) causing PSO precache misses
	//PrecachePSOParams.bAffectDynamicIndirectLighting = false;
	
	bool bHasValidNaniteMaterials = false;
	if (ShouldNaniteSkin() && !ShouldCPUSkin())
	{
		Nanite::FMaterialAudit NaniteMaterials{};
		AuditMaterials(this, NaniteMaterials, true /* Set material usage flags */);
		const bool bIsMaskingAllowed = Nanite::IsMaskingAllowed(GetWorld(), false /* force Nanite for masked */);
		bHasValidNaniteMaterials = NaniteMaterials.IsValid(bIsMaskingAllowed);
	}

	for (FPSOPrecacheVertexFactoryDataPerMaterialIndex& VFsPerMaterial : VFsPerMaterials)
	{
		UMaterialInterface* MaterialInterface = GetMaterial(VFsPerMaterial.MaterialIndex);
		if (MaterialInterface == nullptr)
		{
			MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
		ComponentParams.MaterialInterface = MaterialInterface;
		ComponentParams.VertexFactoryDataList = VFsPerMaterial.VertexFactoryDataList;
		ComponentParams.PSOPrecacheParams = PrecachePSOParams;

		if (bHasValidNaniteMaterials)
		{
			ComponentParams.VertexFactoryDataList.Add(&FNaniteVertexFactory::StaticType);
		}
	}

	TArray<UMaterialInterface*> AllOverlayMaterials;
	TArray<TObjectPtr<UMaterialInterface>> AssetAndComponentMaterialSlotsOverlayMaterial;
	GetMaterialSlotsOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial);
	bool bHaveNullEntry = true;
	FMaterialOverlayHelper::AppendAllOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial, AllOverlayMaterials, bHaveNullEntry);

	// always add default overlay material if available
	if (GetOverlayMaterial())
	{
		AllOverlayMaterials.AddUnique(GetOverlayMaterial());
	}

	if (AllOverlayMaterials.Num() > 0 && VFsPerMaterials.Num() != 0)
	{
		for(UMaterialInterface* OverlayMaterialInterface : AllOverlayMaterials)
		{
			if (!OverlayMaterialInterface)
			{
				continue;
			}
			// Overlay is rendered with the same set of VFs
			FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];

			ComponentParams.MaterialInterface = OverlayMaterialInterface;
			ComponentParams.VertexFactoryDataList = VFsPerMaterials[0].VertexFactoryDataList;
			ComponentParams.PSOPrecacheParams = PrecachePSOParams;
			ComponentParams.PSOPrecacheParams.bCastShadow = false;

			if (bHasValidNaniteMaterials)
			{
				ComponentParams.VertexFactoryDataList.Add(&FNaniteVertexFactory::StaticType);
			}
		}
	}
}

void USkinnedMeshComponent::InitializeComponent()
{
	NotifyIfSkinnedAssetChanged();

	Super::InitializeComponent();
}

void USkinnedMeshComponent::InitializeBeforeRegister()
{
	NotifyIfSkinnedAssetChanged();

	// The reason this happens before register
	// is so that any transform update (or children transform update)
	// won't result in any issues of accessing SpaceBases
	// This isn't really ideal solution because these transform won't have 
	// any valid data yet.

	AnimUpdateRateParams = FAnimUpdateRateManager::GetUpdateRateParameters(this);

	if (LeaderPoseComponent.IsValid())
	{
		// We have to make sure it updates the leader pose
		SetLeaderPoseComponent(LeaderPoseComponent.Get(), true);
	}
	else
	{
		AllocateTransformData();
	}
}

void USkinnedMeshComponent::InitializeOnRegister()
{
	if (FSceneInterface* Scene = GetScene())
	{
		CachedSceneFeatureLevel = Scene->GetFeatureLevel();
	}
	else
	{
		CachedSceneFeatureLevel = ERHIFeatureLevel::Num;
	}

	ForceStreamedLodModel = GetForcedLOD();

#if WITH_EDITOR
	// When we are in editor, ensure that the initial setup is done at LOD 0 so that non-ticking components dont
	// get unexpected behavior when transitioning LODs post-construction script
	const int32 OldForcedLOD = GetForcedLOD();
	if (GIsEditor)
	{
		SetForcedLOD(1);
	}
#endif

	UpdateLODStatus();

#if WITH_EDITOR
	if (GIsEditor)
	{
		SetForcedLOD(OldForcedLOD);
	}
#endif

	InvalidateCachedBounds();
}

void USkinnedMeshComponent::OnPreRegister()
{
	// For more information about the logic in this method, see USkinnedMeshComponent::OnRegister comments.
	TRACE_CPUPROFILER_EVENT_SCOPE(USkinnedMeshComponent::OnPreRegister);

	// Super schedules an async physics job and TransformData must be allocated before that job starts running
	InitializeBeforeRegister();
	
	Super::OnPreRegister();

	InitializeOnRegister();
}

void USkinnedMeshComponent::OnRegister()
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);

	const bool bHasBeenPreRegistered = IsPreRegistered();
	if (!bHasBeenPreRegistered)
	{
		InitializeBeforeRegister();
	}

	Super::OnRegister();

	if (!bHasBeenPreRegistered)
	{
		InitializeOnRegister();
	}

	CreateMeshDeformerInstances(GetActiveMeshDeformers());

	RefreshExternalMorphTargetWeights();

	PrecachePSOs();
}

void USkinnedMeshComponent::OnUnregister()
{
	DeallocateTransformData();
	Super::OnUnregister();

	if (AnimUpdateRateParams)
	{
		FAnimUpdateRateManager::CleanupUpdateRateParametersRef(this);
		AnimUpdateRateParams = nullptr;
	}

	CreateMeshDeformerInstances(FMeshDeformerSet());
}

void USkinnedMeshComponent::BeginPlay()
{
	Super::BeginPlay();

	// Recreate the deformer instances to collect any changes for component bindings during startup.
	CreateMeshDeformerInstances(GetActiveMeshDeformers());
}

const FExternalMorphWeightData& USkinnedMeshComponent::GetExternalMorphWeights(int32 LOD) const
{
	return ExternalMorphWeightData[LOD];
}

FExternalMorphWeightData& USkinnedMeshComponent::GetExternalMorphWeights(int32 LOD)
{
	return ExternalMorphWeightData[LOD];
}

bool USkinnedMeshComponent::IsValidExternalMorphSetLODIndex(int32 LOD) const
{
	return ExternalMorphWeightData.IsValidIndex(LOD) && ExternalMorphSets.IsValidIndex(LOD);
}

void USkinnedMeshComponent::AddExternalMorphSet(int32 LOD, int32 ID, TSharedPtr<FExternalMorphSet> MorphSet)
{
	const int32 NumLODs = GetNumLODs();
	if (NumLODs != ExternalMorphSets.Num())
	{
		ResizeExternalMorphTargetSets();
	}

	if (ExternalMorphSets.IsValidIndex(LOD))
	{
		ExternalMorphSets[LOD].Add(ID, MorphSet);
	}
}

void USkinnedMeshComponent::RemoveExternalMorphSet(int32 LOD, int32 ID)
{
	const int32 NumLODs = GetNumLODs();
	if (NumLODs != ExternalMorphSets.Num())
	{
		ResizeExternalMorphTargetSets();
	}

	if (ExternalMorphSets.IsValidIndex(LOD))
	{
		ExternalMorphSets[LOD].Remove(ID);
	}

	if (ExternalMorphWeightData.IsValidIndex(LOD))
	{
		ExternalMorphWeightData[LOD].MorphSets.Remove(ID);
	}
}

bool USkinnedMeshComponent::HasExternalMorphSet(int32 LOD, int32 ID) const
{
	if (ExternalMorphSets.IsValidIndex(LOD))
	{
		return (ExternalMorphSets[LOD].Find(ID) != nullptr);
	}

	return false;
}

void USkinnedMeshComponent::ClearExternalMorphSets(int32 LOD)
{
	const int32 NumLODs = GetNumLODs();
	if (NumLODs != ExternalMorphSets.Num())
	{
		ResizeExternalMorphTargetSets();
	}

	if (ExternalMorphSets.IsValidIndex(LOD))
	{
		// Remove all weight sets as well.
		if (ExternalMorphWeightData.IsValidIndex(LOD))
		{
			ExternalMorphWeightData[LOD].MorphSets.Empty();
		}

		ExternalMorphSets[LOD].Empty();
	}
}

void USkinnedMeshComponent::ResizeExternalMorphTargetSets()
{
	const int32 NewNumLODs = GetNumLODs();
	if (ExternalMorphSets.Num() == NewNumLODs)
	{
		return;
	}

	if (NewNumLODs > ExternalMorphSets.Num())
	{
		const int32 NumLODsToAdd = NewNumLODs - ExternalMorphSets.Num();
		ExternalMorphSets.AddDefaulted(NumLODsToAdd);
	}
	else
	{
		check(NewNumLODs < ExternalMorphSets.Num());
		const int32 NumLODsToRemove = ExternalMorphSets.Num() - NewNumLODs;
		for (int32 Index = 0; Index < NumLODsToRemove; ++Index)
		{
			ExternalMorphSets.Pop(EAllowShrinking::No);
		}
	}
}

void USkinnedMeshComponent::RefreshExternalMorphTargetWeights(bool bZeroOldWeights)
{
	// Clear the external weights if there is no skeletal mesh.
	if (GetSkinnedAsset() == nullptr)
	{
		ExternalMorphWeightData.Empty();
		ExternalMorphWeightData.AddDefaulted(GetNumLODs());
		return;
	}

	const int32 NumLODs = GetNumLODs();
	if (NumLODs != ExternalMorphSets.Num())
	{
		ResizeExternalMorphTargetSets();
	}
	check(ExternalMorphSets.Num() == NumLODs);

	if (ExternalMorphWeightData.Num() != NumLODs)
	{
		ExternalMorphWeightData.SetNum(NumLODs);
	}

	// Make sure that for every LOD's list of morph target sets, we have the correct number of morph target weights.
	// This resets all weights to 0 as well.
	for (int32 LOD = 0; LOD < NumLODs; ++LOD)
	{
		FExternalMorphWeightData& ExternalWeights = GetExternalMorphWeights(LOD);

		auto& MorphSets = ExternalMorphSets[LOD];
		for (const auto& Item : MorphSets)
		{
			check(Item.Value.IsValid());
			const int32 MorphSetID = Item.Key;
			const FMorphTargetVertexInfoBuffers& MorphBuffer = Item.Value->MorphBuffers;

			FExternalMorphSetWeights* MorphSetWeights = ExternalWeights.MorphSets.Find(MorphSetID);
			if (MorphSetWeights == nullptr)
			{
				MorphSetWeights = &ExternalWeights.MorphSets.Add(MorphSetID);
			}
			check(MorphSetWeights != nullptr);

			MorphSetWeights->Name = Item.Value->Name;
			if (bZeroOldWeights)
			{
				MorphSetWeights->Weights.Reset();
				MorphSetWeights->Weights.AddZeroed(MorphBuffer.GetNumMorphs());
			}
			else
			{
				MorphSetWeights->Weights.SetNumZeroed(MorphBuffer.GetNumMorphs());
			}
		}

		ExternalWeights.UpdateNumActiveMorphTargets();
	}
}

void USkinnedMeshComponent::UpdateBoneTransformRevisionNumber()
{
	if (CurrentBoneTransformFrame != GFrameCounter)
	{
		// We only want to update the previous revision number for the first call to UpdateBoneTransformRevisionNumber on a given frame
		PreviousBoneTransformRevisionNumber = CurrentBoneTransformRevisionNumber;
		CurrentBoneTransformFrame = GFrameCounter;
	}

	if (BoneTransformUpdateMethodQueue.Last() == EBoneTransformUpdateMethod::ClearMotionVector)
	{
		// Last entry is ClearMotionVector, increment revision number by 2 which allows current bone buffer to be bound to previous shader slot to cancel out velocity
		// See FGPUBaseSkinVertexFactory::FShaderDataType::GetBoneBufferInternal() for how it's used.
		CurrentBoneTransformRevisionNumber = PreviousBoneTransformRevisionNumber + 2;
	}
	else
	{
		CurrentBoneTransformRevisionNumber = PreviousBoneTransformRevisionNumber + 1;
	}
}

EPreviousBoneTransformUpdateMode USkinnedMeshComponent::GetPreviousBoneTransformUpdateMode()
{
	EPreviousBoneTransformUpdateMode PreviousUpdateMode = EPreviousBoneTransformUpdateMode::None;

	if (BoneTransformUpdateMethodQueue.Num() > 0)
	{
		if (BoneTransformUpdateMethodQueue.Last() != EBoneTransformUpdateMethod::ClearMotionVector)
		{
			if (BoneTransformUpdateMethodQueue.Num() > 1)
			{
				// When there are multiple entries in the queue, previous bone transforms may or may not need updating, so for simplicity make sure it updates.
				// E.g. Queue: ForceMotionVector -> AnimationUpdate, previous data may be overriden by ForceMotionVector and should update
				PreviousUpdateMode = EPreviousBoneTransformUpdateMode::UpdatePrevious;
			}

			// If we may not evaluate animation every frame and there's an animation update in the queue, copy current to previous
			if (bExternalEvaluationRateLimited && !bExternalInterpolate)
			{
				// Is there an animation update in the queue?
				int32 AnimationUpdateIndex = -1;
				for (int32 i = 0; i < BoneTransformUpdateMethodQueue.Num(); ++i)
				{
					if (BoneTransformUpdateMethodQueue[i] == EBoneTransformUpdateMethod::AnimationUpdate)
					{
						AnimationUpdateIndex = i;
						break;
					}
				}
				if (AnimationUpdateIndex > -1)
				{
					PreviousUpdateMode = EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious;
				}
			}
		}
	}

	return PreviousUpdateMode;
}

void USkinnedMeshComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	UE::TScopeLock Lock(RenderStateLock);


#if WITH_STATE_STREAM_ACTOR
	if (true) // This branch is here to easily toggle between statestream and not.. will be removed once we are on par
	{
		if (GetSkinnedAsset() && FApp::CanEverRender() && ShouldComponentAddToScene())
		{
			FSkinnedMeshStaticState Ss;
			Ss.SetMaterialRelevance(GetMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(ERHIFeatureLevel::SM6)).Raw);
			FSkinnedMeshDynamicState Ds;
			Ds.SetTransform(GetOrCreateTransformHandle());
			Ds.SetSkinnedAsset(GetSkinnedAsset());
			Ds.SetOverrideMaterials(OverrideMaterials);
			MeshHandle = GetWorld()->GetStateStream<ISkinnedMeshStateStream>().Game_CreateInstance(Ss, Ds);
		}
		bRenderStateCreated = true;
		return;
	}
#endif


	if( GetSkinnedAsset() )
	{
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetSkinnedAsset()->GetPackage(), ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(UWorld::StaticClass(), ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, UWorld::StaticClass()->GetFName(), GetSkinnedAsset()->GetPackage()->GetFName());

		// Attempting to track down UE-45505, where it looks as if somehow a skeletal mesh component's mesh has only been partially loaded, causing a mismatch in the LOD arrays
		checkf(!GetSkinnedAsset()->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_WillBeLoaded), TEXT("Attempting to create render state for a skeletal mesh that is is not fully loaded. Mesh: %s"), *GetSkinnedAsset()->GetName());

		// Initialize the alternate weight tracks if present BEFORE creating the new mesh object
		InitLODInfos();

		// No need to create the mesh object if we aren't actually rendering anything (see UPrimitiveComponent::Attach)
		if ( FApp::CanEverRender() && ShouldComponentAddToScene() )
		{
			ERHIFeatureLevel::Type SceneFeatureLevel = GetWorld()->GetFeatureLevel();
			if( FSkeletalMeshRenderData* SkelMeshRenderData = GetSkinnedAsset()->GetResourceForRendering() )
			{
				int32 MinLODIndex = ComputeMinLOD();
			
			#if DO_CHECK
				for (int LODIndex = MinLODIndex; LODIndex < SkelMeshRenderData->LODRenderData.Num(); LODIndex++)
				{
					FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];
					const FPositionVertexBuffer* PositionVertexBufferPtr = &LODData.StaticVertexBuffers.PositionVertexBuffer;
					if (!PositionVertexBufferPtr || (PositionVertexBufferPtr->GetNumVertices() <= 0))
					{
						UE_LOGF(LogSkinnedMeshComp, Warning, "Invalid Lod %i for Rendering Asset: %ls", LODIndex, *GetSkinnedAsset()->GetFullName());
					}
				}
			#endif
	
				if (MeshObjectFactory)
				{
					MeshObject = MeshObjectFactory(MeshObjectFactoryUserData, this, SkelMeshRenderData, SceneFeatureLevel);
				}

				if (!MeshObject && ShouldNaniteSkin() && !ShouldCPUSkin() /* Needed for calls to GetCPUSkinnedVertices() */)
				{
					FSkeletalMeshObjectNanite* NaniteMeshObject = ::new FSkeletalMeshObjectNanite(this, SkelMeshRenderData, SceneFeatureLevel);
					if (NaniteMeshObject->HasValidMaterials())
					{
						MeshObject = NaniteMeshObject;
					}
					else
					{
						NaniteMeshObject->ReleaseResources();
						BeginCleanup(NaniteMeshObject);
					}
				}

				if (!MeshObject)
				{
					if (bRenderStatic)
					{
						// GPU skin vertex buffer + LocalVertexFactory
						MeshObject = ::new FSkeletalMeshObjectStatic(this, SkelMeshRenderData, SceneFeatureLevel);
					}
					else if (ShouldCPUSkin())
					{
						MeshObject = ::new FSkeletalMeshObjectCPUSkin(this, SkelMeshRenderData, SceneFeatureLevel);
					}
					else
					{
						MeshObject = ::new FSkeletalMeshObjectGPUSkin(this, SkelMeshRenderData, SceneFeatureLevel);
					}
				}

				//Allow the editor a chance to manipulate it before its added to the scene
				PostInitMeshObject(MeshObject);
			}
		}
	}

	// Any data needed from PreviousMeshObject will have been propagated in the construction of the new MeshObject above,
	// so we can delete it now.
	if (PreviousMeshObject)
	{
		PreviousMeshObject->ReleaseResources();
		BeginCleanup(PreviousMeshObject);

		PreviousMeshObject = nullptr;
	}

	Super::CreateRenderState_Concurrent(Context);

	int32 UseLOD = 0;

	if (GetSkinnedAsset())
	{
		BoneTransformUpdateMethodQueue.Reset();

		// Update dynamic data
		if(MeshObject)
		{
			// Clamp LOD within the VALID range
			// This is just to re-verify if LOD is WITHIN the valid range
			// Do not replace this with UpdateLODStatus, which could change the LOD 
			//	without animated, causing random skinning issues
			// This can happen if your MinLOD is not valid anymore after loading
			// which causes meshes to be invisible
			const int32 MinLodIndex = ComputeMinLOD();
			const int32 MaxLODIndex = MeshObject->GetSkeletalMeshRenderData().LODRenderData.Num() - 1;
			UseLOD = FMath::Clamp(GetPredictedLODLevel(), MinLodIndex, MaxLODIndex);

			// Clamp to loaded streaming data if available
			if ((GetSkinnedAsset()->IsStreamable() || !IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::SkeletalMesh)) && MeshObject)
			{
				UseLOD = FMath::Max<int32>(UseLOD, MeshObject->GetSkeletalMeshRenderData().PendingFirstLODIdx);
			}

			// If we have a valid LOD, set up required data, during reimport we may try to create data before we have all the LODs
			// imported, in that case we skip until we have all the LODs
			if(GetSkinnedAsset()->IsValidLODIndex(UseLOD))
			{
				const bool bMorphTargetsAllowed = CVarEnableMorphTargets.GetValueOnAnyThread(true) != 0;

				// Are morph targets disabled for this LOD?
				if (bDisableMorphTarget || !bMorphTargetsAllowed)
				{
					ActiveMorphTargets.Empty();
				}

				constexpr bool bZeroOldWeights = false;
				RefreshExternalMorphTargetWeights(bZeroOldWeights);
				MeshObject->Update(UseLOD, this, ActiveMorphTargets, MorphTargetWeights, EPreviousBoneTransformUpdateMode::UpdatePrevious, GetExternalMorphWeights(UseLOD));  // send to rendering thread
			}
		}

		// scene proxy update of material usage based on active morphs
		UpdateMorphMaterialUsageOnProxy();
	}

	for (UMeshDeformerInstance* MeshDeformerInstance : MeshDeformerInstances.DeformerInstances)
	{
		if (MeshDeformerInstance != nullptr)
		{
			MeshDeformerInstance->AllocateResources();
		}
	}

	if (UMeshDeformerInstance* MeshDeformerInstance = GetMeshDeformerInstanceForLOD(UseLOD))
 	{

#if WITH_EDITORONLY_DATA
		for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : GeometryReadbackRequests)
		{
			MeshDeformerInstance->RequestReadbackDeformerGeometry(MoveTemp(Request));
		}
#endif // WITH_EDITORONLY_DATA

		// Enqueue work here to ensure that we have some deformer outputs written for the first frame.
		UMeshDeformerInstance::FEnqueueWorkDesc Desc;
		Desc.Scene = GetScene();
		Desc.OwnerName = GetSkinnedAsset() != nullptr ? GetSkinnedAsset()->GetFName() : GetFName();
		MeshDeformerInstance->EnqueueWork(Desc);
	}

#if WITH_EDITORONLY_DATA
	// Avoid unbound accumulation of readback requests, each readback request must be picked up by the deformer instance within the same frame or ignored otherwise
	GeometryReadbackRequests.Reset();
#endif // WITH_EDITORONLY_DATA
}

void USkinnedMeshComponent::DestroyRenderState_Concurrent()
{
	UE::TScopeLock Lock(RenderStateLock);

#if WITH_STATE_STREAM_ACTOR
	MeshHandle = {};
#endif

	Super::DestroyRenderState_Concurrent();

	ReleaseMeshDeformerInstances();

	if(MeshObject)
	{
		if (bRenderStateRecreating)
		{
			// Preserve previous mesh object for recreate purposes
			PreviousMeshObject = MeshObject;
			MeshObject = nullptr;
		}
		else
		{
			// Begin releasing the RHI resources used by this skeletal mesh component.
			// This doesn't immediately destroy anything, since the rendering thread may still be using the resources.
			MeshObject->ReleaseResources();

			// Begin a deferred delete of MeshObject.  BeginCleanup will call MeshObject->FinishDestroy after the above release resource
			// commands execute in the rendering thread.
			BeginCleanup(MeshObject);
			MeshObject = nullptr;
		}
	}
}

bool USkinnedMeshComponent::RequiresGameThreadEndOfFrameRecreate() const
{
	// When we are a leader/follower, we cannot recreate render state in parallel as this could 
	// happen concurrently with our dependent component(s)
	return LeaderPoseComponent.Get() != nullptr || FollowerPoseComponents.Num() > 0;
}

FString USkinnedMeshComponent::GetDetailedInfoInternal() const
{
	FString Result;  

	if(GetSkinnedAsset() != nullptr )
	{
		Result = GetSkinnedAsset()->GetDetailedInfoInternal();
	}
	else
	{
		Result = TEXT("No_SkeletalMesh");
	}

	return Result;  
}

void USkinnedMeshComponent::UpdateSkinWeightProfileReadbackIfNeeded(int32 LODIndex)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_USkinnedMeshComponent_UpdateSkinWeightProfileReadbackIfNeeded);

	if (bSkinWeightProfilePending && IsSkinnedMeshSkinWeightProfilesOnRenderThread())
	{
		FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();
		if (!SkelMeshRenderData)
		{
			return;
		}

		FSkeletalMeshLODRenderData& LODRenderData = SkelMeshRenderData->LODRenderData[LODIndex];
		if (!LODRenderData.IsDataReady())
		{
			// Data is actively being serialized in a different thread,
			// skip this time.
			return;
		}

		USkinnedAsset* Asset = GetSkinnedAsset();
		if (Asset != nullptr && Asset->HasPendingInitOrStreaming())
		{
			return;
		}

		FSkinWeightProfilesData& SkinWeightProfilesData = LODRenderData.SkinWeightProfilesData;
		const FSkinWeightProfileStack ProfileStack{ CurrentSkinWeightProfileLayers };

		// If a real GPU readback is going to fire, register stream-out protection so the
		// underlying VB resource isn't replaced out from under the captured BaseBuffer*
		// before the RT lambda runs EnqueueCopy.
		if (IStreamingManager::Get_Concurrent() && IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::SkeletalMesh) && SkinWeightProfilesData.ShouldEnqueueGPUReadback())
		{
			IRenderAssetStreamingManager& RAMgr = IStreamingManager::Get().GetRenderAssetStreamingManager();
			if (!RAMgr.TryRegisterStreamOutProtectedAsset(Asset))
			{
				// Pending streaming op in flight; defer to next call.
				// bSkinWeightProfilePending stays true so we retry next frame.
				return;
			}

			// Hand release responsibility to FSkinWeightProfileManager's polling.
			if (FSkinWeightProfileManager* SWPMgr = FSkinWeightProfileManager::Get(GetWorld()))
			{
				SWPMgr->RegisterStreamOutProtection_AsyncSafe(Asset);
			}
		}

		if(SkinWeightProfilesData.DoReadbackIfNeeded(ProfileStack))
		{
			bSkinWeightProfilePending = false;
			bSkinWeightProfileBufferOverridePending = true;
		}
	}
}

void USkinnedMeshComponent::SendRenderDynamicData_Concurrent()
{
	SCOPE_CYCLE_COUNTER(STAT_SkelCompUpdateTransform);


#if WITH_EDITOR
	if (GetSkinnedAsset() && GetSkinnedAsset()->IsCompiling())
	{
		return;
	}
#endif

	EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode = GetPreviousBoneTransformUpdateMode();
	// CurrentBoneTransformRevisionNumber and PreviousBoneTransformUpdateMode are up-to-date at this point, it is safe to reset BoneTransformUpdateMethodQueue
	BoneTransformUpdateMethodQueue.Reset();

	// if we have not updated the transforms then no need to send them to the rendering thread
	// @todo GIsEditor used to be bUpdateSkelWhenNotRendered. Look into it further to find out why it doesn't update animations in the AnimSetViewer, when a level is loaded in UED (like POC_Cover.gear).
	if( MeshObject && GetSkinnedAsset() && (bForceMeshObjectUpdate || (bRecentlyRendered || VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones || GIsEditor || MeshObject->bHasBeenUpdatedAtLeastOnce == false)) )
	{
		SCOPE_CYCLE_COUNTER(STAT_MeshObjectUpdate);

		const int32 MinLodIndex = ComputeMinLOD();
		const int32 MaxLODIndex = MeshObject->GetSkeletalMeshRenderData().LODRenderData.Num() - 1;
		int32 UseLOD = FMath::Clamp(GetPredictedLODLevel(), MinLodIndex, MaxLODIndex);

		// Clamp to loaded streaming data if available
		if ((GetSkinnedAsset()->IsStreamable() || !IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::SkeletalMesh)) && MeshObject)
		{
			UseLOD = FMath::Max<int32>(UseLOD, MeshObject->GetSkeletalMeshRenderData().PendingFirstLODIdx);
		}

		UpdateSkinWeightProfileReadbackIfNeeded(UseLOD);

		// Only update the state if PredictedLODLevel is valid
		FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();
		if (SkelMeshRenderData && SkelMeshRenderData->LODRenderData.IsValidIndex(UseLOD))
		{
			const bool bMorphTargetsAllowed = CVarEnableMorphTargets.GetValueOnAnyThread(true) != 0;

			// Are morph targets disabled for this LOD?
			if (bDisableMorphTarget || !bMorphTargetsAllowed)
			{
				ActiveMorphTargets.Empty();
			}
			
			MeshObject->Update(
				UseLOD,
				this,
				ActiveMorphTargets,
				MorphTargetWeights,
				PreviousBoneTransformUpdateMode,
				GetExternalMorphWeights(UseLOD)
			);  // send to rendering thread

			MeshObject->bHasBeenUpdatedAtLeastOnce = true;
			bForceMeshObjectUpdate = false;

			// scene proxy update of material usage based on active morphs
			UpdateMorphMaterialUsageOnProxy();
		}

		UMeshDeformerInstance* DeformerInstanceForLOD = GetMeshDeformerInstanceForLOD(UseLOD);
		if (DeformerInstanceForLOD)
		{
#if WITH_EDITORONLY_DATA
			for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : GeometryReadbackRequests)
			{
				DeformerInstanceForLOD->RequestReadbackDeformerGeometry(MoveTemp(Request));
			}
#endif // WITH_EDITORONLY_DATA
			
			UMeshDeformerInstance::FEnqueueWorkDesc Desc;
			Desc.Scene = GetScene();
			Desc.OwnerName = GetSkinnedAsset() != nullptr ? GetSkinnedAsset()->GetFName() : GetFName();
			// Fallback is to reset the passthrough vertex factory if the deformer fails to run.
			Desc.FallbackDelegate.BindLambda([MeshObjectPtr = MeshObject, UseLOD]() { FSkeletalMeshDeformerHelpers::ResetVertexFactoryBufferOverrides(MeshObjectPtr, UseLOD); });

			DeformerInstanceForLOD->EnqueueWork(Desc);
		}
	}

	Super::SendRenderDynamicData_Concurrent();

#if WITH_EDITORONLY_DATA
	// Avoid unbound accumulation of readback requests, each readback request must be picked up by the deformer instance within the same frame or ignored otherwise
	GeometryReadbackRequests.Reset();
#endif // WITH_EDITORONLY_DATA 
}

void USkinnedMeshComponent::ClearMotionVector()
{
	if (MeshObject)
	{
		// if you have situation where you want to clear the bone velocity (that causes temporal AA or motion blur)
		// use this function to clear it
		BoneTransformUpdateMethodQueue.Add(EBoneTransformUpdateMethod::ClearMotionVector);
		// Similar to FlipEditableSpaceBases(), update revision number before reaching dynamic update to make sure dependency between lead and follower components are preserved.
		UpdateBoneTransformRevisionNumber();
		// Make sure an update happens
		bForceMeshObjectUpdate = true;
		MarkRenderDynamicDataDirty();
	}
}

void USkinnedMeshComponent::ForceMotionVector()
{
	if (MeshObject)
	{
		BoneTransformUpdateMethodQueue.Add(EBoneTransformUpdateMethod::ForceMotionVector);
		// Similar to FlipEditableSpaceBases(), update revision number before reaching dynamic update to make sure dependency between lead and follower components are preserved.
		UpdateBoneTransformRevisionNumber();
		// Make sure an update happens
		bForceMeshObjectUpdate = true;
		MarkRenderDynamicDataDirty();
	}
}

#if WITH_EDITOR

bool USkinnedMeshComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USkinnedMeshComponent, bCastCapsuleIndirectShadow))
		{
			return CastShadow && bCastDynamicShadow;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(USkinnedMeshComponent, CapsuleIndirectShadowMinVisibility))
		{
			return bCastCapsuleIndirectShadow && CastShadow && bCastDynamicShadow;
		}
	}

	return Super::CanEditChange(InProperty);
}

void USkinnedMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	NotifyIfSkinnedAssetChanged();

	if (const FProperty* Property = PropertyChangedEvent.Property)
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(USkinnedMeshComponent, MeshDeformer) ||
			Property->GetFName() == GET_MEMBER_NAME_CHECKED(USkinnedMeshComponent, bSetMeshDeformer) ||
			Property->GetFName() == GET_MEMBER_NAME_CHECKED(USkinnedMeshComponent, bAlwaysUseMeshDeformer))
		{
			const FMeshDeformerSet ActiveDeformers = GetActiveMeshDeformers();

			// Only one deformer is supported for now
			check(ActiveDeformers.Deformers.Num() <= 1);
			UMeshDeformer* ActiveMeshDeformer = ActiveDeformers.Deformers.Num() > 0 ? ActiveDeformers.Deformers[0] : nullptr;

			MeshDeformerInstanceSettings = ActiveMeshDeformer ? ActiveMeshDeformer->CreateSettingsInstance(this) : nullptr;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void USkinnedMeshComponent::PostEditUndo()
{
	NotifyIfSkinnedAssetChanged();

	Super::PostEditUndo();

	// Post undo, make sure that any socket attachments are up to date.
	UpdateCachedSocketAttachmentFlag();
}

void USkinnedMeshComponent::PostDuplicate(bool bDuplicateForPIE)
{
	NotifyIfSkinnedAssetChanged();

	Super::PostDuplicate(bDuplicateForPIE);
}

void USkinnedMeshComponent::PostEditImport()
{
	NotifyIfSkinnedAssetChanged();

	Super::PostEditImport();
}

EDataValidationResult USkinnedMeshComponent::IsDataValid(class FDataValidationContext& Context) const
{
	EDataValidationResult Result = EDataValidationResult::NotValidated;

	if (USkinnedAsset* Asset = GetSkinnedAsset())
	{
		FMeshDeformerSet MeshDeformers = GetActiveMeshDeformers();

		for (int32 LODIndex = 0; LODIndex < MeshDeformers.DeformerIndexForLOD.Num(); LODIndex++)
		{
			int32 DeformerIndex = MeshDeformers.DeformerIndexForLOD[LODIndex];

			if (DeformerIndex == INDEX_NONE)
			{
				// No deformer is needed for this LOD
				continue;
			}
			else if (!MeshDeformers.DeformerIndexForLOD.IsValidIndex(DeformerIndex))
			{
				// Deformer index for this LOD is invalid
				Context.AddWarning(
					FText::Format(
						LOCTEXT("InvalidDeformerIndexForLOD",
								"The deformer index {0} is invalid for Skeletal Mesh Asset {1} as "
								"there are only {2} deformers available."),
						DeformerIndex,
						FText::FromName(Asset->GetPackage()->GetFName()),
						MeshDeformers.Deformers.Num()
					));
				Result = EDataValidationResult::Invalid;
				break;
			}

			UMeshDeformer* Deformer = MeshDeformers.Deformers[DeformerIndex];
			if (Cast<ISkeletalMeshHalfEdgeBufferAccessor>(Deformer))
			{
				if (SkeletalMeshHalfEdgeBufferAccessor::IsHalfEdgeRequired(Deformer))
				{
					if (!Asset->HasHalfEdgeBuffer(LODIndex))
					{
						// Deformer requires the half edge buffer to work,
						// but it is missing on the skinned asset,
						// so notify the user to manual fix it
						Context.AddWarning(
							FText::Format(
								LOCTEXT("SkeletalMeshMissingHalfEdgeBuffer",
								        "Please add Deformer Graph {0} to TargetMeshDeformers of Skeletal Mesh Asset {1} to "
								        "make sure extra data required by the Deformer is cooked for non-editor builds."),
								FText::FromName(Deformer->GetPackage()->GetFName()),
								FText::FromName(Asset->GetPackage()->GetFName())
							));
						Result = EDataValidationResult::Invalid;
						break;
					}
				}
			}
		}
	}
	
	return CombineDataValidationResults(Result, Super::IsDataValid(Context));
}

#endif // WITH_EDITOR

void USkinnedMeshComponent::InitLODInfos()
{
	if (GetSkinnedAsset() != nullptr)
	{
		if (GetSkinnedAsset()->GetLODNum() != LODInfo.Num())
		{
			// Perform cleanup if LOD infos have been initialized before 
			if (!LODInfo.IsEmpty() && GReleasePreviousLODInfoOnInitialization)
			{
				// Batch release all resources of LOD infos we're about to destruct.
				// This is relevant when overrides have been set but LODInfos are
				// re-initialized, for example by changing the mesh at runtime.
				bool bNeedRenderFlush = false;
				
				for (int32 Idx = 0; Idx < LODInfo.Num(); ++Idx)
				{
					if (LODInfo[Idx].BeginReleaseOverrideSkinWeights())
					{
						bNeedRenderFlush = true;
					}
					
					if (LODInfo[Idx].BeginReleaseOverrideVertexColors())
					{
						bNeedRenderFlush = true;
					}
				}

				if (bNeedRenderFlush)
				{
					FlushRenderingCommands();
					for (int32 Idx = 0; Idx < LODInfo.Num(); ++Idx)
					{
						LODInfo[Idx].EndReleaseOverrideSkinWeights();
						LODInfo[Idx].EndReleaseOverrideVertexColors();
					}
				}
			}
			
			LODInfo.Empty(GetSkinnedAsset()->GetLODNum());
			for (int32 Idx=0; Idx < GetSkinnedAsset()->GetLODNum(); Idx++)
			{
				new(LODInfo) FSkelMeshComponentLODInfo();
			}
		}
	}	
}


bool USkinnedMeshComponent::ShouldTickPose() const
{
	if (LeaderPoseComponent.IsValid() && !bFollowerShouldTickPose)
	{
		return false;
	}

	return ((VisibilityBasedAnimTickOption < EVisibilityBasedAnimTickOption::OnlyTickPoseWhenRendered) || bRecentlyRendered);
}

bool USkinnedMeshComponent::ShouldUpdateTransform(bool bLODHasChanged) const
{
	return (bRecentlyRendered || (VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones));
}

bool USkinnedMeshComponent::ShouldUseUpdateRateOptimizations() const
{
	return bEnableUpdateRateOptimizations && CVarEnableAnimRateOptimization.GetValueOnAnyThread() > 0;
}

void USkinnedMeshComponent::TickPose(float DeltaTime, bool bNeedsValidRootMotion)
{
	OnTickPose.Broadcast(this, DeltaTime, bNeedsValidRootMotion);
	TickUpdateRate(DeltaTime, bNeedsValidRootMotion);
}

void USkinnedMeshComponent::TickUpdateRate(float DeltaTime, bool bNeedsValidRootMotion)
{
	SCOPE_CYCLE_COUNTER(STAT_TickUpdateRate);
	if (ShouldUseUpdateRateOptimizations())
	{
		if (GetOwner())
		{
			// Tick Owner once per frame. All attached SkinnedMeshComponents will share the same settings.
			FAnimUpdateRateManager::TickUpdateRateParameters(this, DeltaTime, bNeedsValidRootMotion);

#if ENABLE_DRAW_DEBUG
			if ((CVarDrawAnimRateOptimization.GetValueOnGameThread() > 0) || bDisplayDebugUpdateRateOptimizations)
			{
				FColor DrawColor = AnimUpdateRateParams->GetUpdateRateDebugColor();
				DrawDebugBox(GetWorld(), Bounds.Origin, Bounds.BoxExtent, FQuat::Identity, DrawColor, false);

				FString DebugString = FString::Printf(TEXT("%s UpdateRate(%d) EvaluationRate(%d) ShouldInterpolateSkippedFrames(%d) ShouldSkipUpdate(%d) Interp Alpha (%f) AdditionalTime(%f)"),
					*GetNameSafe(GetSkinnedAsset()), AnimUpdateRateParams->UpdateRate, AnimUpdateRateParams->EvaluationRate,
					AnimUpdateRateParams->ShouldInterpolateSkippedFrames(), AnimUpdateRateParams->ShouldSkipUpdate(), AnimUpdateRateParams->GetInterpolationAlpha(), AnimUpdateRateParams->AdditionalTime);

				GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.f, FColor::Red, DebugString, false);
			}
#endif // ENABLE_DRAW_DEBUG
		}
	}
}

void USkinnedMeshComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	SCOPED_NAMED_EVENT(USkinnedMeshComponent_TickComponent, FColor::Yellow);
	SCOPE_CYCLE_COUNTER(STAT_SkinnedMeshCompTick);

	// We cant run the blueprint events (ReceiveTick, latent actions) on worker threads, so skip them if we are running as such
	if(!PrimaryComponentTick.bRunOnAnyThread)
	{
		// Tick ActorComponent first.
		Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	}

	// See if this mesh was rendered recently. This has to happen first because other data will rely on this
#if WITH_STATE_STREAM_ACTOR
	bRecentlyRendered = true;
#else
	bRecentlyRendered = ((bUseScreenRenderStateForUpdate ? GetLastRenderTimeOnScreen() : GetLastRenderTime()) > GetWorld()->TimeSeconds - 1.0f);
#endif

	// Update component's LOD settings
	// This must be done BEFORE animation Update and Evaluate (TickPose and RefreshBoneTransforms respectively)
	const bool bLODHasChanged = UpdateLODStatus();

	// Skip the rest of the work if we are running on worker threads as none of it is safe
	if(!PrimaryComponentTick.bRunOnAnyThread)
	{
		// Tick Pose first
		if (ShouldTickPose())
		{
			TickPose(DeltaTime, false);
		}

		// If we have been recently rendered, and bForceRefPose has been on for at least a frame, or the LOD changed, update bone matrices.
		if( ShouldUpdateTransform(bLODHasChanged) )
		{
			// Do not update bones if we are taking bone transforms from another SkelMeshComp
			if( LeaderPoseComponent.IsValid() )
			{
				UpdateFollowerComponent();
			}
			else 
			{
				RefreshBoneTransforms(ThisTickFunction);
			}
		}
		else if(VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPose)
		{
			// We are not refreshing bone transforms, but we do want to tick pose. We may need to kick off a parallel task
			DispatchParallelTickPose(ThisTickFunction);
		}
#if WITH_EDITOR
		else 
		{
			// only do this for level viewport actors
			UWorld* World = GetWorld();
			if (World && World->WorldType == EWorldType::Editor)
			{
				RefreshMorphTargets();
			}
		}
#endif // WITH_EDITOR
		
		if (IsSkinnedMeshSkinWeightProfilesOnRenderThread() && bSkinWeightProfileBufferOverridePending)
		{
			// Update the skin weight override buffer once GPU Readback of weight data is done. 
			// As it cannot be updated during send dynamic render data, we override the buffer
			// here. 
			UpdateSkinWeightOverrideBuffer();
			bSkinWeightProfileBufferOverridePending = false;
			bSkinWeightProfileSet = true;
		}
	}
}

UObject const* USkinnedMeshComponent::AdditionalStatObject() const
{
	return GetSkinnedAsset();
}

void USkinnedMeshComponent::UpdateFollowerComponent()
{
	MarkRenderDynamicDataDirty();
}

// this has to be skeletalmesh material. You can't have more than what SkeletalMesh materials have
int32 USkinnedMeshComponent::GetNumMaterials() const
{
	if (GetSkinnedAsset())
	{
		return GetSkinnedAsset()->GetMaterials().Num();
	}

	return 0;
}

void USkinnedMeshComponent::GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const
{
	const TIndirectArray<FSkeletalMeshLODRenderData>& LODRenderData = GetSkeletalMeshRenderData()->LODRenderData;
	const TArray<FSkeletalMaterial>& Materials = GetSkinnedAsset()->GetMaterials();
	PrimitiveStats.LODStats.Reserve(LODRenderData.Num());
	int32 LOD = 0;
	for (const FSkeletalMeshLODRenderData& RenderData : LODRenderData)
	{
		FPrimitiveLODStats& LODStats = PrimitiveStats.LODStats.EmplaceAt_GetRef(LOD, LOD);
		LODStats.Sections = RenderData.RenderSections.Num();
		LODStats.bIsOptionalLOD = RenderData.bIsLODOptional;
		LODStats.bIsAvailable = RenderData.bStreamedDataInlined || RenderData.StreamingBulkData.DoesExist();
		FResourceSizeEx ResourceSize;
		RenderData.GetResourceSizeEx(ResourceSize);
		LODStats.TotalResourceSize = ResourceSize.GetTotalMemoryBytes();
		for (const auto& Section : RenderData.RenderSections)
		{
			if (Materials[Section.MaterialIndex].MaterialInterface)
			{
				LODStats.MaterialIndices.Add(Section.MaterialIndex);
			}
			LODStats.Triangles += Section.NumTriangles;
		}
		LOD++;
	}
}

UMaterialInterface* USkinnedMeshComponent::GetMaterial(int32 MaterialIndex) const
{
	return FSkinnedMeshComponentHelper::GetMaterial(*this, MaterialIndex);
}

int32 USkinnedMeshComponent::GetMaterialIndex(FName MaterialSlotName) const
{
	if (GetSkinnedAsset())
	{
		const TArray<FSkeletalMaterial>& SkeletalMeshMaterials = GetSkinnedAsset()->GetMaterials();
		for (int32 MaterialIndex = 0; MaterialIndex < SkeletalMeshMaterials.Num(); ++MaterialIndex)
		{
			const FSkeletalMaterial &SkeletalMaterial = SkeletalMeshMaterials[MaterialIndex];
			if (SkeletalMaterial.MaterialSlotName == MaterialSlotName)
			{
				return MaterialIndex;
			}
		}
	}
	return INDEX_NONE;
}

TArray<FName> USkinnedMeshComponent::GetMaterialSlotNames() const
{
	TArray<FName> MaterialNames;
	if (GetSkinnedAsset())
	{
		const TArray<FSkeletalMaterial>& SkeletalMeshMaterials = GetSkinnedAsset()->GetMaterials();
		for (int32 MaterialIndex = 0; MaterialIndex < SkeletalMeshMaterials.Num(); ++MaterialIndex)
		{
			const FSkeletalMaterial &SkeletalMaterial = SkeletalMeshMaterials[MaterialIndex];
			MaterialNames.Add(SkeletalMaterial.MaterialSlotName);
		}
	}
	return MaterialNames;
}

bool USkinnedMeshComponent::IsMaterialSlotNameValid(FName MaterialSlotName) const
{
	return GetMaterialIndex(MaterialSlotName) >= 0;
}

bool USkinnedMeshComponent::ShouldNaniteSkin()
{
	return FSkinnedMeshComponentHelper::ShouldNaniteSkin(*this);
}

bool USkinnedMeshComponent::ShouldCPUSkin() const
{
	return GetCPUSkinningEnabled();
}

bool USkinnedMeshComponent::GetCPUSkinningEnabled() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return bCPUSkinning;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void USkinnedMeshComponent::SetCPUSkinningEnabled(bool bEnable, bool bRecreateRenderStateImmediately)
{
	checkSlow(IsInGameThread());
	check(GetSkinnedAsset() && GetSkinnedAsset()->GetResourceForRendering());

	if (GetCPUSkinningEnabled() == bEnable)
	{
		return;
	}

	if (bEnable && IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::SkeletalMesh))
	{
		UE_LOGF(LogSkinnedMeshComp, Warning, "It is expensive to enable CPU skinning with LOD streaming on.");

		IRenderAssetStreamingManager& Manager = IStreamingManager::Get().GetRenderAssetStreamingManager();
		Manager.BlockTillAllRequestsFinished();

		const bool bOriginalForcedFullyLoad = GetSkinnedAsset()->bForceMiplevelsToBeResident;
		GetSkinnedAsset()->bForceMiplevelsToBeResident = true;
		Manager.UpdateIndividualRenderAsset(GetSkinnedAsset());

		GetSkinnedAsset()->WaitForPendingInitOrStreaming();

		check(GetSkinnedAsset()->GetResourceForRendering()->CurrentFirstLODIdx <= GetSkinnedAsset()->GetDefaultMinLod());

		GetSkinnedAsset()->UnlinkStreaming();
		GetSkinnedAsset()->bForceMiplevelsToBeResident = bOriginalForcedFullyLoad;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bCPUSkinning = bEnable;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	if (IsRegistered())
	{
		if (bRecreateRenderStateImmediately)
		{
			RecreateRenderState_Concurrent();
			FlushRenderingCommands();
		}
		else
		{
			MarkRenderStateDirty();
		}
	}
}

bool USkinnedMeshComponent::GetMaterialStreamingData(int32 MaterialIndex, FPrimitiveMaterialInfo& MaterialData) const
{
	if (GetSkinnedAsset() && GetSkinnedAsset()->IsMaterialUsed(MaterialIndex))
	{
		MaterialData.Material = GetMaterial(MaterialIndex);
		MaterialData.UVChannelData = GetSkinnedAsset()->GetUVChannelData(MaterialIndex);
		MaterialData.PackedRelativeBox = PackedRelativeBox_Identity;
	}
	return MaterialData.IsValid();
}

void USkinnedMeshComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	GetStreamingTextureInfoInner(LevelContext, nullptr, StreamingDistanceMultiplier, OutStreamingRenderAssets);

	if (GetSkinnedAsset() && GetSkinnedAsset()->IsStreamable())
	{
		const int32 LocalForcedLodModel = GetForceStreamedLOD();

		// A positive texel factor tells the streaming system the size of the component.
		// 
		// A negative texel factor selects a specific LOD index, starting at -1 for the worst LOD
		const float TexelFactor = LocalForcedLodModel >= 0 ? LocalForcedLodModel - GetSkinnedAsset()->GetLODNum() : Bounds.SphereRadius * 2.f;

		new (OutStreamingRenderAssets) FStreamingRenderAssetPrimitiveInfo(GetSkinnedAsset(), Bounds, TexelFactor, PackedRelativeBox_Identity, false, false);
	}
}

void USkinnedMeshComponent::OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex)
{
	if (!IsComponentTickEnabled())
	{
		check(RenderAsset == GetSkinnedAsset());

		if (GetPredictedLODLevel() < FirstLodIndex)
		{
			SetPredictedLODLevel(FirstLodIndex);
			bForceMeshObjectUpdate = true;
			MarkRenderDynamicDataDirty();
		}
	}
}

bool USkinnedMeshComponent::ShouldUpdateBoneVisibility() const
{
	// do not update if it has LeaderPoseComponent
	return !LeaderPoseComponent.IsValid();
}
void USkinnedMeshComponent::RebuildVisibilityArray()
{
	// BoneVisibility needs update if LeaderComponent == nullptr
	// if MaterComponent, it should follow MaterPoseComponent
	if ( ShouldUpdateBoneVisibility())
	{
		// If the BoneVisibilityStates array has a 0 for a parent bone, all children bones are meant to be hidden as well
		// (as the concatenated matrix will have scale 0).  This code propagates explicitly hidden parents to children.

		// On the first read of any cell of BoneVisibilityStates, BVS_HiddenByParent and BVS_Visible are treated as visible.
		// If it starts out visible, the value written back will be BVS_Visible if the parent is visible; otherwise BVS_HiddenByParent.
		// If it starts out hidden, the BVS_ExplicitlyHidden value stays in place

		// The following code relies on a complete hierarchy sorted from parent to children
		TArray<uint8>& EditableBoneVisibilityStates = GetEditableBoneVisibilityStates();
		if (EditableBoneVisibilityStates.Num() != GetSkinnedAsset()->GetRefSkeleton().GetNum())
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "RebuildVisibilityArray() failed because EditableBoneVisibilityStates size: %d not equal to RefSkeleton bone count: %d.", EditableBoneVisibilityStates.Num(), GetSkinnedAsset()->GetRefSkeleton().GetNum());
			return;
		}

		for (int32 BoneId=0; BoneId < EditableBoneVisibilityStates.Num(); ++BoneId)
		{
			uint8 VisState = EditableBoneVisibilityStates[BoneId];

			// if not exclusively hidden, consider if parent is hidden
			if (VisState != BVS_ExplicitlyHidden)
			{
				// Check direct parent (only need to do one deep, since we have already processed the parent and written to BoneVisibilityStates previously)
				const int32 ParentIndex = GetSkinnedAsset()->GetRefSkeleton().GetParentIndex(BoneId);
				if ((ParentIndex == -1) || (EditableBoneVisibilityStates[ParentIndex] == BVS_Visible))
				{
					EditableBoneVisibilityStates[BoneId] = BVS_Visible;
				}
				else
				{
					EditableBoneVisibilityStates[BoneId] = BVS_HiddenByParent;
				}
			}
		}

		bBoneVisibilityDirty = true;
	}
}

FBoxSphereBounds USkinnedMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	SCOPE_CYCLE_COUNTER(STAT_CalcSkelMeshBounds);

	return bIncludeComponentLocationIntoBounds ?
		CalcMeshBound(FVector3f::ZeroVector, false, LocalToWorld) + FBoxSphereBounds(GetComponentLocation(), FVector(1.0f), 1.0f) :
		CalcMeshBound(FVector3f::ZeroVector, false, LocalToWorld);
}

void USkinnedMeshComponent::UpdateBounds()
{
	// UpdateBounds is currently the closest place we can get from the SkinnedAsset property
	// being overwritten by a CDO construction from SpawnActor because PostInitProperties is 
	// not currently being called during subobjects property copies.
	NotifyIfSkinnedAssetChanged();

	if (LastStreamerUpdateBoundsRadius < 0.f)
	{
		LastStreamerUpdateBoundsRadius = float(Bounds.SphereRadius);
	}

#if WITH_EDITOR
	//We cannot update bounds during compilation
	if (GetSkinnedAsset() && GetSkinnedAsset()->IsCompiling())
	{
		return;
	}
#endif

	Super::UpdateBounds();

	// Only notify the streamer if the bounds radius has changed enough
	if (GUpdateBoundsNotifyStreamingRadiusChangeRatio >= 0.f
		&& FMath::Abs(float(Bounds.SphereRadius) - LastStreamerUpdateBoundsRadius) > GUpdateBoundsNotifyStreamingRadiusChangeRatio * FMath::Min(float(Bounds.SphereRadius), LastStreamerUpdateBoundsRadius)
		&& GetForcedLOD() == 0)
	{
		IStreamingManager::Get().NotifyPrimitiveUpdated_Concurrent(this);
		LastStreamerUpdateBoundsRadius = float(Bounds.SphereRadius);
	}
}

class UPhysicsAsset* USkinnedMeshComponent::GetPhysicsAsset() const
{
	if (PhysicsAssetOverride)
	{
		return PhysicsAssetOverride;
	}

	if (GetSkinnedAsset() && GetSkinnedAsset()->GetPhysicsAsset())
	{
		return GetSkinnedAsset()->GetPhysicsAsset();
	}

	return nullptr;
}

FBoxSphereBounds USkinnedMeshComponent::CalcMeshBound(const FVector3f& RootOffset, bool UsePhysicsAsset, const FTransform& LocalToWorld) const
{
	FBoxSphereBounds NewBounds;

	// If physics are asleep, and actor is using physics to move, skip updating the bounds.
	AActor* Owner = GetOwner();
	FVector DrawScale = LocalToWorld.GetScale3D();	

	const USkinnedMeshComponent* const LeaderPoseComponentInst = LeaderPoseComponent.Get();
	UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();
	UPhysicsAsset * const LeaderPhysicsAsset = (LeaderPoseComponentInst != nullptr)? LeaderPoseComponentInst->GetPhysicsAsset() : nullptr;

	// Can only use the PhysicsAsset to calculate the bounding box if we are not non-uniformly scaling the mesh.
	const USkinnedAsset* SkinnedAssetConst = GetSkinnedAsset();
	const bool bCanUsePhysicsAsset = DrawScale.IsUniform() && (GetSkinnedAsset() != nullptr)
		// either space base exists or child component
		&& ( (GetNumComponentSpaceTransforms() == SkinnedAssetConst->GetRefSkeleton().GetNum()) || (LeaderPhysicsAsset) );

	const bool bDetailModeAllowsRendering = (DetailMode <= GetCachedScalabilityCVars().DetailMode);
	const bool bIsVisible = ( bDetailModeAllowsRendering && (ShouldRender() || bCastHiddenShadow));

	const bool bHasPhysBodies = PhysicsAsset && PhysicsAsset->SkeletalBodySetups.Num();
	const bool bLeaderHasPhysBodies = LeaderPhysicsAsset && LeaderPhysicsAsset->SkeletalBodySetups.Num();

	// if not visible, or we were told to use fixed bounds, use skelmesh bounds
	if ( (!bIsVisible || bComponentUseFixedSkelBounds) && GetSkinnedAsset())
	{
		FBoxSphereBounds RootAdjustedBounds = SkinnedAssetConst->GetBounds();
		RootAdjustedBounds.Origin += FVector(RootOffset); // Adjust bounds by root bone translation
		NewBounds = RootAdjustedBounds.TransformBy(LocalToWorld);
	}
	else if(LeaderPoseComponentInst && LeaderPoseComponentInst->GetSkinnedAsset() && LeaderPoseComponentInst->bComponentUseFixedSkelBounds)
	{
		FBoxSphereBounds RootAdjustedBounds = LeaderPoseComponentInst->GetSkinnedAsset()->GetBounds();
		RootAdjustedBounds.Origin += FVector(RootOffset); // Adjust bounds by root bone translation
		NewBounds = RootAdjustedBounds.TransformBy(LocalToWorld);
	}
	// Use LeaderPoseComponent's PhysicsAsset if told to
	else if (LeaderPoseComponentInst && bCanUsePhysicsAsset && bUseBoundsFromLeaderPoseComponent)
	{
		NewBounds = LeaderPoseComponentInst->CalcBounds(LocalToWorld);
	}
#if WITH_EDITOR
	// For AnimSet Viewer, use 'bounds preview' physics asset if present.
	else if(GetSkinnedAsset() && bHasPhysBodies && bCanUsePhysicsAsset && PhysicsAsset->CanCalculateValidAABB(this, LocalToWorld))
	{
		NewBounds = FBoxSphereBounds(PhysicsAsset->CalcAABB(this, LocalToWorld));
	}
#endif // WITH_EDITOR
	// If we have a PhysicsAsset (with at least one matching bone), and we can use it, do so to calc bounds.
	else if( bHasPhysBodies && bCanUsePhysicsAsset && UsePhysicsAsset )
	{
		NewBounds = FBoxSphereBounds(PhysicsAsset->CalcAABB(this, LocalToWorld));
	}
	// Use LeaderPoseComponent's PhysicsAsset, if we don't have one and it does
	else if(LeaderPoseComponentInst && bCanUsePhysicsAsset && bLeaderHasPhysBodies)
	{
		NewBounds = FBoxSphereBounds(LeaderPhysicsAsset->CalcAABB(this, LocalToWorld));
	}
	// Fallback is to use the one from the skeletal mesh. Usually pretty bad in terms of Accuracy of where the SkelMesh Bounds are located (i.e. usually bigger than it needs to be)
	else if(GetSkinnedAsset())
	{
		FBoxSphereBounds RootAdjustedBounds = GetSkinnedAsset()->GetBounds();

		// Adjust bounds by root bone translation
		RootAdjustedBounds.Origin += FVector(RootOffset);
		NewBounds = RootAdjustedBounds.TransformBy(LocalToWorld);
	}
	else
	{
		NewBounds = FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}

	// TODO - Add bounds of any per-poly collision data.

	NewBounds.BoxExtent *= BoundsScale;
	NewBounds.SphereRadius *= BoundsScale;

	return NewBounds;
}

void USkinnedMeshComponent::GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const
{
#if WITH_EDITOR
	// We cannot read the asset's bounds while it is compiling. Reaching into USkinnedAsset::GetBounds()
	// during a LockPropertiesUntil() can self-deadlock the GT against its own parked async-build worker
	// (e.g. UGroomComponent::CalcBounds running from SendAllEndOfFrameUpdates while a reimport holds the lock).
	// Match the pattern already enforced by USkinnedMeshComponent::UpdateBounds and SendRenderDynamicData_Concurrent.
	if (IsCompiling())
	{
		OutBounds = FBoxSphereBounds(ForceInitToZero);
		return;
	}
#endif

	const USkinnedMeshComponent* const LeaderPoseComponentInst = LeaderPoseComponent.Get();

	if (GetSkinnedAsset())
	{
		// Get the Pre-skinned bounds from the skeletal mesh. Note that these bounds are the "ExtendedBounds", so they can be tweaked on the SkeletalMesh
		OutBounds = GetSkinnedAsset()->GetBounds();
	}
	else if(LeaderPoseComponentInst && LeaderPoseComponentInst->GetSkinnedAsset())
	{
#if WITH_EDITOR
		if (LeaderPoseComponentInst->IsCompiling())
		{
			OutBounds = FBoxSphereBounds(ForceInitToZero);
			return;
		}
#endif
		// Get the bounds from the leader pose if ther is no skeletal mesh
		OutBounds = LeaderPoseComponentInst->GetSkinnedAsset()->GetBounds();
	}
	else
	{
		// Fall back
		OutBounds = FBoxSphereBounds(ForceInitToZero);
	}
}

bool USkinnedMeshComponent::AreBoneTransformsValid() const
{
	return IsRegistered();
}

FMatrix USkinnedMeshComponent::GetBoneMatrix(int32 BoneIdx) const
{
	if (!AreBoneTransformsValid())
	{
		// if not registered, we don't have SpaceBases yet. 
		// also GetComponentTransform() isn't set yet (They're set from relativetranslation, relativerotation, relativescale)
		return FMatrix::Identity;
	}

	// Handle case of use a LeaderPoseComponent - get bone matrix from there.
	const USkinnedMeshComponent* const LeaderPoseComponentInst = LeaderPoseComponent.Get();
	if(LeaderPoseComponentInst)
	{
		if(BoneIdx < GetLeaderBoneMap().Num())
		{
			int32 ParentBoneIndex = GetLeaderBoneMap()[BoneIdx];

			// If ParentBoneIndex is valid, grab matrix from LeaderPoseComponent.
			if(	ParentBoneIndex != INDEX_NONE && 
				ParentBoneIndex < LeaderPoseComponentInst->GetNumComponentSpaceTransforms())
			{
				return LeaderPoseComponentInst->GetComponentSpaceTransforms()[ParentBoneIndex].ToMatrixWithScale() * GetComponentTransform().ToMatrixWithScale();
			}
			else
			{
				UE_LOGF(LogSkinnedMeshComp, Verbose, "GetBoneMatrix : ParentBoneIndex(%d:%ls) out of range of LeaderPoseComponent->SpaceBases for %ls(%ls)", BoneIdx, *GetNameSafe(LeaderPoseComponentInst->GetSkinnedAsset()), *GetNameSafe(GetSkinnedAsset()), *GetPathName());
				return FMatrix::Identity;
			}
		}
		else
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "GetBoneMatrix : BoneIndex(%d) out of range of LeaderBoneMap for %ls (%ls)", BoneIdx, *this->GetFName().ToString(), GetSkinnedAsset() ? *GetSkinnedAsset()->GetFName().ToString() : TEXT("NULL") );
			return FMatrix::Identity;
		}
	}
	else
	{
		const int32 NumTransforms = GetNumComponentSpaceTransforms();
		if(BoneIdx >= 0 && BoneIdx < NumTransforms)
		{
			return GetComponentSpaceTransforms()[BoneIdx].ToMatrixWithScale() * GetComponentTransform().ToMatrixWithScale();
		}
		else
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "GetBoneMatrix : BoneIndex(%d) out of range of SpaceBases for %ls (%ls)", BoneIdx, *GetPathName(), GetSkinnedAsset() ? *GetSkinnedAsset()->GetFullName() : TEXT("NULL") );
			return FMatrix::Identity;
		}
	}
}

FTransform USkinnedMeshComponent::GetBoneTransform(int32 BoneIdx) const
{
	if (!AreBoneTransformsValid())
	{
		// if not registered, we don't have SpaceBases yet. 
		// also GetComponentTransform() isn't set yet (They're set from relativelocation, relativerotation, relativescale)
		return FTransform::Identity;
	}

	return GetBoneTransform(BoneIdx, GetComponentTransform());
}

FTransform USkinnedMeshComponent::GetBoneTransform(int32 BoneIdx, const FTransform& LocalToWorld) const
{
	// Handle case of use a LeaderPoseComponent - get bone matrix from there.
	const USkinnedMeshComponent* const LeaderPoseComponentInst = LeaderPoseComponent.Get();
	if(LeaderPoseComponentInst)
	{
		if (!LeaderPoseComponentInst->AreBoneTransformsValid())
		{
			// We aren't going to get anything valid from the leader pose if it
			// isn't valid so for now return identity
			return FTransform::Identity;
		}
		if(BoneIdx >= 0 && BoneIdx < GetLeaderBoneMap().Num())
		{
			const int32 LeaderBoneIndex = GetLeaderBoneMap()[BoneIdx];
			const int32 NumLeaderTransforms = LeaderPoseComponentInst->GetNumComponentSpaceTransforms();

			// If LeaderBoneIndex is valid, grab matrix from LeaderPoseComponent.
			if(LeaderBoneIndex >= 0 && LeaderBoneIndex < NumLeaderTransforms)
			{
				return LeaderPoseComponentInst->GetComponentSpaceTransforms()[LeaderBoneIndex] * LocalToWorld;
			}
			else
			{
				// Is this a missing bone we have cached?
				FMissingLeaderBoneCacheEntry MissingBoneInfo;
				const FMissingLeaderBoneCacheEntry* MissingBoneInfoPtr = MissingLeaderBoneMap.Find(BoneIdx);
				if(MissingBoneInfoPtr != nullptr)
				{
					const int32 MissingLeaderBoneIndex = MissingBoneInfoPtr->CommonAncestorBoneIndex;
					if(MissingLeaderBoneIndex >= 0 && MissingLeaderBoneIndex < NumLeaderTransforms)
					{
						return MissingBoneInfoPtr->RelativeTransform * LeaderPoseComponentInst->GetComponentSpaceTransforms()[MissingBoneInfoPtr->CommonAncestorBoneIndex] * LocalToWorld;
					}
				}
				// Otherwise we might be able to generate the missing transform on the fly (although this is expensive)
				else if(GetMissingLeaderBoneRelativeTransform(BoneIdx, MissingBoneInfo))
				{
					const int32 MissingLeaderBoneIndex = MissingBoneInfo.CommonAncestorBoneIndex;
					if (MissingLeaderBoneIndex >= 0 && MissingLeaderBoneIndex < NumLeaderTransforms)
					{
						return MissingBoneInfo.RelativeTransform * LeaderPoseComponentInst->GetComponentSpaceTransforms()[MissingBoneInfo.CommonAncestorBoneIndex] * LocalToWorld;
					}
				}

				UE_LOGF(LogSkinnedMeshComp, Verbose, "GetBoneTransform : ParentBoneIndex(%d) out of range of LeaderPoseComponent->SpaceBases for %ls", BoneIdx, *this->GetFName().ToString() );
				return FTransform::Identity;
			}
		}
		else
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "GetBoneTransform : BoneIndex(%d) out of range of LeaderBoneMap for %ls", BoneIdx, *this->GetFName().ToString() );
			return FTransform::Identity;
		}
	}
	else
	{
		const int32 NumTransforms = GetNumComponentSpaceTransforms();
		if(BoneIdx >= 0 && BoneIdx < NumTransforms)
		{
			return GetComponentSpaceTransforms()[BoneIdx] * LocalToWorld;
		}
		else
		{
			UE_LOGF(LogSkinnedMeshComp, Verbose, "GetBoneTransform : BoneIndex(%d) out of range of SpaceBases for %ls (%ls)", BoneIdx, *GetPathName(), GetSkinnedAsset() ? *GetSkinnedAsset()->GetFullName() : TEXT("NULL") );
			return FTransform::Identity;
		}
	}
}

int32 USkinnedMeshComponent::GetNumBones()const
{
	return GetSkinnedAsset() ? GetSkinnedAsset()->GetRefSkeleton().GetNum() : 0;
}

int32 USkinnedMeshComponent::GetBoneIndex( FName BoneName) const
{
	int32 BoneIndex = INDEX_NONE;
	if ( BoneName != NAME_None && GetSkinnedAsset())
	{
		BoneIndex = GetSkinnedAsset()->GetRefSkeleton().FindBoneIndex( BoneName );
	}

	return BoneIndex;
}


FName USkinnedMeshComponent::GetBoneName(int32 BoneIndex) const
{
	return (GetSkinnedAsset() != nullptr && GetSkinnedAsset()->GetRefSkeleton().IsValidIndex(BoneIndex)) ? GetSkinnedAsset()->GetRefSkeleton().GetBoneName(BoneIndex) : NAME_None;
}


FName USkinnedMeshComponent::GetParentBone( FName BoneName ) const
{
	FName Result = NAME_None;

	int32 BoneIndex = GetBoneIndex(BoneName);
	if ((BoneIndex != INDEX_NONE) && (BoneIndex > 0)) // This checks that this bone is not the root (ie no parent), and that BoneIndex != INDEX_NONE (ie bone name was found)
	{
		Result = GetSkinnedAsset()->GetRefSkeleton().GetBoneName(GetSkinnedAsset()->GetRefSkeleton().GetParentIndex(BoneIndex));
	}
	return Result;
}

FTransform USkinnedMeshComponent::GetDeltaTransformFromRefPose(FName BoneName, FName BaseName/* = NAME_None*/) const
{
	if (GetSkinnedAsset())
	{
		const FReferenceSkeleton& RefSkeleton = GetSkinnedAsset()->GetRefSkeleton();
		const int32 BoneIndex = GetBoneIndex(BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			FTransform CurrentTransform = GetBoneTransform(BoneIndex);
			FTransform ReferenceTransform = FAnimationRuntime::GetComponentSpaceTransformRefPose(RefSkeleton, BoneIndex);
			if (BaseName == NAME_None)
			{
				BaseName = GetParentBone(BoneName);
			}

			const int32 BaseIndex = GetBoneIndex(BaseName);
			if (BaseIndex != INDEX_NONE)
			{	
				CurrentTransform = CurrentTransform.GetRelativeTransform(GetBoneTransform(BaseIndex));
				ReferenceTransform = ReferenceTransform.GetRelativeTransform(FAnimationRuntime::GetComponentSpaceTransformRefPose(RefSkeleton, BaseIndex));
			}

			// get delta of two transform
			return CurrentTransform.GetRelativeTransform(ReferenceTransform);
		}
	}

	return FTransform::Identity;
}

bool USkinnedMeshComponent::GetTwistAndSwingAngleOfDeltaRotationFromRefPose(FName BoneName, float& OutTwistAngle, float& OutSwingAngle) const
{
	const FReferenceSkeleton& RefSkeleton = GetSkinnedAsset()->GetRefSkeleton();
	const int32 BoneIndex = GetBoneIndex(BoneName);
	const TArray<FTransform>& Transforms = GetComponentSpaceTransforms();

	// detect the case where we don't have a pose yet
	if (Transforms.Num() == 0)
	{
		OutTwistAngle = OutSwingAngle = 0.f;
		return false;
	}

	if (BoneIndex != INDEX_NONE && ensureMsgf(BoneIndex < Transforms.Num(), TEXT("Invalid transform access in %s. Index=%d, Num=%d"), *GetPathName(), BoneIndex, Transforms.Num()))
	{
		FTransform LocalTransform = GetComponentSpaceTransforms()[BoneIndex];
		FTransform ReferenceTransform = RefSkeleton.GetRefBonePose()[BoneIndex];
		FName ParentName = GetParentBone(BoneName);
		int32 ParentIndex = INDEX_NONE;
		if (ParentName != NAME_None)
		{
			ParentIndex = GetBoneIndex(ParentName);
		}

		if (ParentIndex != INDEX_NONE)
		{
			LocalTransform = LocalTransform.GetRelativeTransform(GetComponentSpaceTransforms()[ParentIndex]);
		}

		FQuat Swing, Twist;

		// figure out based on ref pose rotation, and calculate twist based on that 
		FVector TwistAxis = ReferenceTransform.GetRotation().Vector();
		ensure(TwistAxis.IsNormalized());
		LocalTransform.GetRotation().ToSwingTwist(TwistAxis, Swing, Twist);
		OutTwistAngle = FMath::RadiansToDegrees(Twist.GetAngle());
		OutSwingAngle = FMath::RadiansToDegrees(Swing.GetAngle());
		return true;
	}

	return false;
}

bool USkinnedMeshComponent::IsSkinCacheAllowed(int32 LodIdx) const
{
	ERHIFeatureLevel::Type FeatureLevel = GetScene() ? GetScene()->GetFeatureLevel() : GMaxRHIFeatureLevel;
	return IsSkinCacheAllowed(LodIdx, FeatureLevel);
}

bool USkinnedMeshComponent::IsSkinCacheAllowed(int32 LodIdx, ERHIFeatureLevel::Type FeatureLevel) const
{
	if (!FGPUSkinCache::IsGPUSkinCacheEnabled(FeatureLevel))
	{
		return false;
	}

	if (GetMeshDeformerInstanceForLOD(LodIdx) != nullptr)
	{
		// Disable skin cache if a mesh deformer is in use.
		// Any animation buffers are expected to be owned by the MeshDeformer.
		return false;
	}

	const bool bGlobalDefault = GetSkinCacheDefaultBehavior() == ESkinCacheDefaultBehavior::Inclusive;

	if (!GetSkinnedAsset())
	{
		return bGlobalDefault;
	}

	if (FSkeletalMeshRenderData* SkelMeshRenderData = GetSkinnedAsset()->GetResourceForRendering())
	{
		if (SkelMeshRenderData->bSupportRayTracing && FGPUSkinCache::IsGPUSkinCacheRayTracingSupported(FeatureLevel) && IsRayTracingEnabled())
		{
			return true;
		}
	}

	FSkeletalMeshLODInfo* LodInfo = GetSkinnedAsset()->GetLODInfo(LodIdx);
	if (!LodInfo)
	{
		return bGlobalDefault;
	}

	bool bLodEnabled = LodInfo->SkinCacheUsage == ESkinCacheUsage::Auto ?
		bGlobalDefault :
		LodInfo->SkinCacheUsage == ESkinCacheUsage::Enabled;

	if (!SkinCacheUsage.IsValidIndex(LodIdx))
	{
		return bLodEnabled;
	}

	bool bComponentEnabled = SkinCacheUsage[LodIdx] == ESkinCacheUsage::Auto ? 
		bLodEnabled :
		SkinCacheUsage[LodIdx] == ESkinCacheUsage::Enabled;

	return bComponentEnabled;
}

bool USkinnedMeshComponent::HasMeshDeformer() const
{
	return GetActiveMeshDeformers().Deformers.Num() > 0;
}

bool USkinnedMeshComponent::HasMeshDeformer(int32 LodIdx) const
{
	if (!HasMeshDeformer())
	{
		return false;
	}

	const FMeshDeformerSet& Set = GetActiveMeshDeformers();
	if (!Set.DeformerIndexForLOD.IsValidIndex(LodIdx))
	{
		return false;
	}
	return Set.DeformerIndexForLOD[LodIdx] != INDEX_NONE;
}

void USkinnedMeshComponent::GetBoneNames(TArray<FName>& BoneNames)
{
	if (GetSkinnedAsset() == nullptr)
	{
		// no mesh, so no bones
		BoneNames.Empty();
	}
	else
	{
		// pre-size the array to avoid unnecessary reallocation
		BoneNames.Empty(GetSkinnedAsset()->GetRefSkeleton().GetNum());
		BoneNames.AddUninitialized(GetSkinnedAsset()->GetRefSkeleton().GetNum());
		for (int32 i = 0; i < GetSkinnedAsset()->GetRefSkeleton().GetNum(); i++)
		{
			BoneNames[i] = GetSkinnedAsset()->GetRefSkeleton().GetBoneName(i);
		}
	}
}


bool USkinnedMeshComponent::BoneIsChildOf(FName BoneName, FName ParentBoneName) const
{
	bool bResult = false;

	if(GetSkinnedAsset())
	{
		const int32 BoneIndex = GetSkinnedAsset()->GetRefSkeleton().FindBoneIndex(BoneName);
		if(BoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogSkinnedMeshComp, Log, "execBoneIsChildOf: BoneName '%ls' not found in SkeletalMesh '%ls'", *BoneName.ToString(), *GetSkinnedAsset()->GetName());
			return bResult;
		}

		const int32 ParentBoneIndex = GetSkinnedAsset()->GetRefSkeleton().FindBoneIndex(ParentBoneName);
		if(ParentBoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogSkinnedMeshComp, Log, "execBoneIsChildOf: ParentBoneName '%ls' not found in SkeletalMesh '%ls'", *ParentBoneName.ToString(), *GetSkinnedAsset()->GetName());
			return bResult;
		}

		bResult = GetSkinnedAsset()->GetRefSkeleton().BoneIsChildOf(BoneIndex, ParentBoneIndex);
	}

	return bResult;
}


FVector USkinnedMeshComponent::GetRefPosePosition(int32 BoneIndex) const
{
	if(GetSkinnedAsset() && (BoneIndex >= 0) && (BoneIndex < GetSkinnedAsset()->GetRefSkeleton().GetNum()))
	{
		return GetSkinnedAsset()->GetRefSkeleton().GetRefBonePose()[BoneIndex].GetTranslation();
	}
	else
	{
		return FVector::ZeroVector;
	}
}

FTransform USkinnedMeshComponent::GetRefPoseTransform(int32 BoneIndex) const
{
	if(GetSkinnedAsset() && (BoneIndex >= 0) && (BoneIndex < GetSkinnedAsset()->GetRefSkeleton().GetNum()))
	{
		return GetSkinnedAsset()->GetRefSkeleton().GetRefBonePose()[BoneIndex];
	}
	else
	{
		return FTransform::Identity;
	}
}

void USkinnedMeshComponent::SetSkeletalMesh(USkeletalMesh* NewMesh, bool bReinitPose)
{
	SetSkinnedAssetAndUpdate(NewMesh, bReinitPose);
}

USkeletalMesh* USkinnedMeshComponent::GetSkeletalMesh_DEPRECATED() const
{
	return Cast<class USkeletalMesh>(GetSkinnedAsset());
}

void USkinnedMeshComponent::SetSkeletalMesh_DEPRECATED(USkeletalMesh* NewMesh)
{
	SetSkinnedAssetAndUpdate(Cast<USkinnedAsset>(NewMesh));
}

void USkinnedMeshComponent::SetSkinnedAssetAndUpdate(USkinnedAsset* InSkinnedAsset, bool /*bReinitPose*/)
{
	// NOTE: InSkinnedAsset may be nullptr (useful in the editor for removing the skeletal mesh associated with
	//   this component on-the-fly)

	if (InSkinnedAsset == GetSkinnedAsset())
	{
		// do nothing if the input mesh is the same mesh we're already using.
		return;
	}

	{
		//Handle destroying and recreating the renderstate
		FRenderStateRecreator RenderStateRecreator(this);

		// Ensure we clear all skin weight profiles and requests, since any that might be in flight could 
		// get applied to this new mesh on a LOD switch.
		ClearAllSkinWeightProfiles();

		SetSkinnedAsset(InSkinnedAsset);
		NotifyIfSkinnedAssetChanged();
		SetPredictedLODLevel(0);

		//FollowerPoseComponents is an array of weak obj ptrs, so it can contain null elements
		for (auto Iter = FollowerPoseComponents.CreateIterator(); Iter; ++Iter)
		{
			TWeakObjectPtr<USkinnedMeshComponent> Comp = (*Iter);
			if (Comp.IsValid() == false)
			{
				FollowerPoseComponents.RemoveAt(Iter.GetIndex());
				--Iter;
			}
			else
			{
				Comp->UpdateLeaderBoneMap();
			}
		}

		// Don't init anim state if not registered
		if (IsRegistered() || IsPreRegistering())
		{
			AllocateTransformData();
			UpdateLeaderBoneMap();
			InvalidateCachedBounds();
			// clear morphtarget cache
			ActiveMorphTargets.Empty();
			MorphTargetWeights.Empty();
			ExternalMorphWeightData.Empty();
		}

		PrecachePSOs();

		// Re-init the MeshDeformers which might come from the SkelMesh.
		CreateMeshDeformerInstances(GetActiveMeshDeformers());
	}

	// Update external weight array sizes.
	RefreshExternalMorphTargetWeights();
	
	if (IsRegistered() || IsPreRegistering())
	{
		// We do this after the FRenderStateRecreator has gone as
		// UpdateLODStatus needs a valid MeshObject
		UpdateLODStatus(); 
	}
}

void USkinnedMeshComponent::SetSkinnedAsset(class USkinnedAsset* InSkinnedAsset)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkinnedAsset = InSkinnedAsset;
	SkeletalMesh = Cast<USkeletalMesh>(InSkinnedAsset);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

USkinnedAsset* USkinnedMeshComponent::GetSkinnedAsset() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	USkinnedAsset* CurrentSkinnedAsset = !SkeletalMesh ? SkinnedAsset : SkeletalMesh;	// Return the SkeletalMesh pointer for backward compatibility until it is retired
#if WITH_EDITOR
	// This should never happen and is a last resort, we should have caught the property overwrite well before we reach this code
	if (KnownSkinnedAsset != CurrentSkinnedAsset)
	{
		OutdatedKnownSkinnedAssetDetected();
	}
#endif
	return CurrentSkinnedAsset;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FMeshDeformerSet USkinnedMeshComponent::GetActiveMeshDeformers() const
{
	const FSkeletalMeshRenderData* RenderData = GetSkeletalMeshRenderData();

	if (!RenderData
		|| !GetSkinnedAsset())
	{
		// In order to select the right deformer, there needs to be a valid mesh and valid render data.
		return FMeshDeformerSet();
	}

	UMeshDeformer* ActiveDeformer = nullptr;
	if (bSetMeshDeformer)
	{
		ActiveDeformer = MeshDeformer;
	}
	else
	{
		ActiveDeformer = GetSkinnedAsset()->GetDefaultMeshDeformer();
	}

	const bool bIsDeformerRequestedByUser = ActiveDeformer != nullptr;

	// If there's no user-specified deformer, find out if we need to set a default deformer for
	// Unlimited Bone Influences.
	EShaderPlatform ShaderPlatformToCheck = GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel];
	if (GetScene())
	{
		ShaderPlatformToCheck = GetScene()->GetShaderPlatform();
	}
	
	const bool bIsDeformerRequiredForUBI = FGPUBaseSkinVertexFactory::GetAlwaysUseDeformerForUnlimitedBoneInfluences(ShaderPlatformToCheck);

	if (!bIsDeformerRequestedByUser)
	{
		bool bShouldStillUseMeshDeformer = false;
		
		if (bAlwaysUseMeshDeformer)
		{
			bShouldStillUseMeshDeformer = true;
		}
		else if (bIsDeformerRequiredForUBI)
		{
			bool bMeshUsesUBI = false;
			for (const FSkeletalMeshLODRenderData& LODRenderData : RenderData->LODRenderData)
			{
				if (LODRenderData.IsDataReady() && 
					LODRenderData.SkinWeightVertexBuffer.GetBoneInfluenceType() == UnlimitedBoneInfluence)
				{
					bMeshUsesUBI = true;
					break;
				}
			}

			if (bMeshUsesUBI)
			{
				bShouldStillUseMeshDeformer = true;
			}
		}

		if (bShouldStillUseMeshDeformer)
		{
			static IMeshDeformerProvider* MeshDeformerProvider = IMeshDeformerProvider::Get();

			if (MeshDeformerProvider)
			{
				IMeshDeformerProvider::FDefaultMeshDeformerSetup Setup;
				Setup.bIsRequestingDeformer = true;

				ActiveDeformer = MeshDeformerProvider->GetDefaultMeshDeformer(Setup);
				if (!ActiveDeformer)
				{
					UE_LOGF(LogSkinnedMeshComp, Error, "Failed to fetch default mesh deformer for a mesh that requires it: %ls. "
						"Ensure DefaultDeformer is set to a valid Deformer Graph asset in the project settings", *GetSkinnedAsset()->GetPathName());
				}
			}
		}
	}

	if (!ActiveDeformer)
	{
		return FMeshDeformerSet();
	}

	// Currently only one active deformer is supported
	FMeshDeformerSet Result;
	Result.Deformers.Empty(1);
	Result.Deformers.Add(ActiveDeformer);

	const int32 NumLODs = GetSkinnedAsset()->GetLODNum();
	const int32 MaxLOD = GetMeshDeformerMaxLOD();

	// Every entry of this array will be written by the loop below
	Result.DeformerIndexForLOD.Empty(RenderData->LODRenderData.Num());
	Result.DeformerIndexForLOD.AddUninitialized(RenderData->LODRenderData.Num());

	for (int32 Index = 0; Index < RenderData->LODRenderData.Num(); Index++)
	{
		const FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[Index];
		const bool bRequiredForUBI = bIsDeformerRequiredForUBI && LODRenderData.SkinWeightVertexBuffer.GetBoneInfluenceType() == UnlimitedBoneInfluence;

		const bool bAllowedByMaxLOD = Index <= MaxLOD;
		// There should be a LODInfo entry for this LOD, but if not, default to allowing the deformer
		const bool bAllowedByLODInfo = Index >= NumLODs || GetSkinnedAsset()->GetLODInfo(Index)->bAllowMeshDeformer;

		// Always enable for UBI requests, conditional if it is either user requested or triggered by AlwaysUseMeshDeformer 
		const bool bDeformerEnabledForThisLOD = bRequiredForUBI || ((bIsDeformerRequestedByUser || bAlwaysUseMeshDeformer) && bAllowedByMaxLOD && bAllowedByLODInfo);
		Result.DeformerIndexForLOD[Index] = bDeformerEnabledForThisLOD ? 0 : INDEX_NONE;
	}

	return Result;
}

void USkinnedMeshComponent::CreateMeshDeformerInstances(const FMeshDeformerSet& DeformerSet)
{
	ReleaseMeshDeformerInstances();

	// Only one deformer is supported for now
	check(DeformerSet.Deformers.Num() <= 1);
	UMeshDeformer* ActiveMeshDeformer = DeformerSet.Deformers.Num() > 0 ? DeformerSet.Deformers[0] : nullptr;

	UMeshDeformerInstance* MeshDeformerInstance = ActiveMeshDeformer ? ActiveMeshDeformer->CreateInstance(this, MeshDeformerInstanceSettings) : nullptr;
	
	MeshDeformerInstances = FMeshDeformerInstanceSet();
	if (MeshDeformerInstance)
	{
		MeshDeformerInstances.DeformerInstances.Empty(1);
		MeshDeformerInstances.DeformerInstances.Add(MeshDeformerInstance);

		MeshDeformerInstances.InstanceIndexForLOD = DeformerSet.DeformerIndexForLOD;
	}
}

void USkinnedMeshComponent::ReleaseMeshDeformerInstances()
{
	for (UMeshDeformerInstance* MeshDeformerInstance : MeshDeformerInstances.DeformerInstances)
	{
		if (MeshDeformerInstance != nullptr)
		{
			MeshDeformerInstance->ReleaseResources();
		}
	}
}

UMeshDeformerInstance* USkinnedMeshComponent::GetMeshDeformerInstance() const
{
	return MeshDeformerInstances.DeformerInstances.Num() > 0 ? MeshDeformerInstances.DeformerInstances[0]->GetInstanceForSourceDeformer() : nullptr;
}

UMeshDeformerInstanceSettings* USkinnedMeshComponent::GetMeshDeformerInstanceSettings() const
{
	return MeshDeformerInstanceSettings;
}

UMeshDeformerInstance* USkinnedMeshComponent::GetMeshDeformerInstanceForLOD(int32 LODIndex) const
{
	if (!MeshDeformerInstances.InstanceIndexForLOD.IsValidIndex(LODIndex))
	{
		return nullptr;
	}
	
	const int8 InstanceIndex = MeshDeformerInstances.InstanceIndexForLOD[LODIndex];
	if (InstanceIndex == INDEX_NONE)
	{
		// Don't use a deformer for this LOD
		return nullptr;
	}

	check(MeshDeformerInstances.DeformerInstances.IsValidIndex(InstanceIndex));
	return MeshDeformerInstances.DeformerInstances[InstanceIndex];
}

void USkinnedMeshComponent::SetMeshDeformer(bool bInSetMeshDeformer, UMeshDeformer* InMeshDeformer)
{
	bSetMeshDeformer = bInSetMeshDeformer;
	MeshDeformer = InMeshDeformer;

	const FMeshDeformerSet ActiveDeformers = GetActiveMeshDeformers();

	// Only one deformer is supported for now
	check(ActiveDeformers.Deformers.Num() <= 1);
	UMeshDeformer* ActiveMeshDeformer = ActiveDeformers.Deformers.Num() > 0 ? ActiveDeformers.Deformers[0] : nullptr;

	MeshDeformerInstanceSettings = ActiveMeshDeformer ? ActiveMeshDeformer->CreateSettingsInstance(this) : nullptr;

	CreateMeshDeformerInstances(ActiveDeformers);

	MarkRenderStateDirty();
}

void USkinnedMeshComponent::SetMeshDeformer(UMeshDeformer* InMeshDeformer)
{
	SetMeshDeformer(true, InMeshDeformer);
}

void USkinnedMeshComponent::UnsetMeshDeformer()
{
	SetMeshDeformer(false, nullptr);
}

void USkinnedMeshComponent::SetAlwaysUseMeshDeformer(bool bShouldAlwaysUseMeshDeformer)
{
	bAlwaysUseMeshDeformer = bShouldAlwaysUseMeshDeformer;
	// Refresh active deformer 
	SetMeshDeformer(bSetMeshDeformer, MeshDeformer);
}

bool USkinnedMeshComponent::GetAlwaysUseMeshDeformer() const
{
	return bAlwaysUseMeshDeformer;
}

TAutoConsoleVariable<int32> CVarMeshDeformerMaxLod(
	TEXT("r.MeshDeformerMaxLOD"),
	-1,
	TEXT("Don't apply MeshDeformers to SkinnedMeshs above this LOD.\n")
	TEXT("Default is -1 (disabled).\n"),
	ECVF_Default);

int32 USkinnedMeshComponent::GetMeshDeformerMaxLOD() const
{
	int32 MaxLod = CVarMeshDeformerMaxLod.GetValueOnGameThread();
	return MaxLod >= 0 ? MaxLod : GetNumLODs() - 1;
}

#if WITH_EDITORONLY_DATA
bool USkinnedMeshComponent::RequestReadbackRenderGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest)
{
	if (!IsInGameThread())
	{
		return false;
	}

	// At the moment only support mesh deformed by mesh deformer
	if (!HasMeshDeformer())
	{
		return false;
	}
	
	// These readback requests will be picked up by the deformer instance during CreateRenderStage/SendRenderDynamicData
	GeometryReadbackRequests.Add(MoveTemp(InRequest));
	MarkRenderDynamicDataDirty();
	
	return true;
}
#endif //WITH_EDITORONLY_DATA

FSkeletalMeshRenderData* USkinnedMeshComponent::GetSkeletalMeshRenderData() const
{
	return FSkinnedMeshComponentHelper::GetSkeletalMeshRenderData(*this);
}

void USkinnedMeshComponent::GetUpdateClothSimulationData_AnyThread(TMap<int32, FClothSimulData>& OutClothSimulData, FMatrix& OutLocalToWorld, float& OutClothBlendWeight) const
{
	OutClothSimulData.Reset();
	OutLocalToWorld = FMatrix::Identity;
	OutClothBlendWeight = 0.f;
}

bool USkinnedMeshComponent::AllocateTransformData()
{
	LLM_SCOPE_BYNAME(TEXT("SkeletalMesh/TransformData"));

	// Allocate transforms if not present.
	if (GetSkinnedAsset() != nullptr && LeaderPoseComponent == nullptr )
	{
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetSkinnedAsset()->GetPackage(), ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(UWorld::StaticClass(), ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, UWorld::StaticClass()->GetFName(), GetSkinnedAsset()->GetPackage()->GetFName());

		const int32 NumBones = GetSkinnedAsset()->GetRefSkeleton().GetNum();

		if(GetNumComponentSpaceTransforms() != NumBones)
		{
			for (int32 BaseIndex = 0; BaseIndex < 2; ++BaseIndex)
			{
				ComponentSpaceTransformsArray[BaseIndex].Empty(NumBones);
				ComponentSpaceTransformsArray[BaseIndex].AddUninitialized(NumBones);

				for (int32 I = 0; I < NumBones; ++I)
				{
					ComponentSpaceTransformsArray[BaseIndex][I].SetIdentity();
				}

				BoneVisibilityStates[BaseIndex].Empty(NumBones);
				if(NumBones)
				{
					BoneVisibilityStates[BaseIndex].AddUninitialized(NumBones);
					for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
					{
						BoneVisibilityStates[BaseIndex][BoneIndex] = BVS_Visible;
					}
				}
			}
 
			// when initialize bone transform first time
			// it is invalid
			bHasValidBoneTransform = false;

			// Init previous arrays only if we are not using double-buffering
			if(!bDoubleBufferedComponentSpaceTransforms)
			{
				PreviousComponentSpaceTransformsArray = ComponentSpaceTransformsArray[0];
				PreviousBoneVisibilityStates = BoneVisibilityStates[0];
			}
		}

		// if it's same, do not touch, and return
		return true;
	}
	
	// Reset the animation stuff when changing mesh.
	ComponentSpaceTransformsArray[0].Empty();
	ComponentSpaceTransformsArray[1].Empty();
	PreviousComponentSpaceTransformsArray.Empty();

	return false;
}

void USkinnedMeshComponent::DeallocateTransformData()
{
	ComponentSpaceTransformsArray[0].Empty();
	ComponentSpaceTransformsArray[1].Empty();
	PreviousComponentSpaceTransformsArray.Empty();
	BoneVisibilityStates[0].Empty();
	BoneVisibilityStates[1].Empty();
	PreviousBoneVisibilityStates.Empty();
}

void USkinnedMeshComponent::SetPhysicsAsset(class UPhysicsAsset* InPhysicsAsset, bool bForceReInit)
{
	PhysicsAssetOverride = InPhysicsAsset;
}

void USkinnedMeshComponent::SetLeaderPoseComponent(class USkinnedMeshComponent* NewLeaderBoneComponent, bool bForceUpdate, bool bInFollowerShouldTickPose)
{
	// Early out if we're already setup.
	if (!bForceUpdate && NewLeaderBoneComponent == LeaderPoseComponent)
	{
		return;
	}

	bFollowerShouldTickPose = bInFollowerShouldTickPose;

	USkinnedMeshComponent* OldLeaderPoseComponent = LeaderPoseComponent.Get();
	USkinnedMeshComponent* ValidNewLeaderPose = NewLeaderBoneComponent;

	// now add to follower components list, 
	if (ValidNewLeaderPose)
	{
		// verify if my current Leader pose is valid
		// we can't have chain of Leader poses, so 
		// we'll find the root Leader pose component
		USkinnedMeshComponent* Iterator = ValidNewLeaderPose;
		while (Iterator->LeaderPoseComponent.IsValid())
		{
			ValidNewLeaderPose = Iterator->LeaderPoseComponent.Get();
			Iterator = ValidNewLeaderPose;

			// we have cycling, where in this chain, if it comes back to me, then reject it
			if (Iterator == this)
			{
				ensureAlwaysMsgf(false,
					TEXT("SetLeaderPoseComponent detected loop (the input Leader pose chain point to itself. (%s <- %s)). Aborting... "),
					*GetNameSafe(NewLeaderBoneComponent), *GetNameSafe(this));
				ValidNewLeaderPose = nullptr;
				break;
			}
		}

		// if we have valid Leader pose, compare with input data and we warn users
		if (ValidNewLeaderPose)
		{
			// Output if Leader is not same as input, which means it has changed. 
			UE_CLOGF(ValidNewLeaderPose == NewLeaderBoneComponent, LogSkinnedMeshComp, Verbose,
				"LeaderPoseComponent chain is detected (%ls). We re-route to top-most LeaderPoseComponent (%ls)",
				*GetNameSafe(ValidNewLeaderPose), *GetNameSafe(NewLeaderBoneComponent));
		}
	}

	// now we have valid Leader pose, set it
	LeaderPoseComponent = ValidNewLeaderPose;
	if (ValidNewLeaderPose)
	{
		bool bAddNew = true;
		// make sure no empty element is there, this is weak obj ptr, so it will go away unless there is 
		// other reference, this is intentional as Leader to follower reference is weak
		for (auto Iter = ValidNewLeaderPose->FollowerPoseComponents.CreateIterator(); Iter; ++Iter)
		{
			TWeakObjectPtr<USkinnedMeshComponent> Comp = (*Iter);
			if (Comp.IsValid() == false)
			{
				// remove
				ValidNewLeaderPose->FollowerPoseComponents.RemoveAt(Iter.GetIndex());
				--Iter;
			}
			// if it has same as me, ignore to add
			else if (Comp.Get() == this)
			{
				bAddNew = false;
			}
		}

		if (bAddNew)
		{
			ValidNewLeaderPose->AddFollowerPoseComponent(this);
		}

		// set up tick dependency between leader & follower components
		PrimaryComponentTick.AddPrerequisite(ValidNewLeaderPose, ValidNewLeaderPose->PrimaryComponentTick);
	}

	if ((OldLeaderPoseComponent != nullptr) && (OldLeaderPoseComponent != ValidNewLeaderPose))
	{
		OldLeaderPoseComponent->RemoveFollowerPoseComponent(this);

		// Only remove tick dependency if the old leader pose comp isn't our attach parent. We should always have a tick dependency with our parent (see USceneComponent::AttachToComponent)
		if (GetAttachParent() != OldLeaderPoseComponent)
		{
			// remove tick dependency between leader & follower components
			PrimaryComponentTick.RemovePrerequisite(OldLeaderPoseComponent, OldLeaderPoseComponent->PrimaryComponentTick);
		}
	}

	AllocateTransformData();
	RecreatePhysicsState();
	UpdateLeaderBoneMap();

	// Update Follower in case Leader has already been ticked, and we won't get an update for another frame.
	if (ValidNewLeaderPose)
	{
		// if I have leader, but I also have followers, they won't work anymore
		// we have to reroute the followers to new leader
		if (FollowerPoseComponents.Num() > 0)
		{
			UE_LOGF(LogSkinnedMeshComp, Verbose,
				"LeaderPoseComponent chain is detected (%ls). We re-route all children to new LeaderPoseComponent (%ls)",
				*GetNameSafe(this), *GetNameSafe(ValidNewLeaderPose));

			// Walk through array in reverse, as changing the Followers' LeaderPoseComponent will remove them from our FollowerPoseComponents array.
			const int32 NumFollowers = FollowerPoseComponents.Num();
			for (int32 FollowerIndex = NumFollowers - 1; FollowerIndex >= 0; FollowerIndex--)
			{
				if (USkinnedMeshComponent* FollowerComp = FollowerPoseComponents[FollowerIndex].Get())
				{
					FollowerComp->SetLeaderPoseComponent(ValidNewLeaderPose);
				}
			}
		}

		UpdateFollowerComponent();
	}
}

const TArray< TWeakObjectPtr<USkinnedMeshComponent> >& USkinnedMeshComponent::GetFollowerPoseComponents() const
{
	return FollowerPoseComponents;
}

void USkinnedMeshComponent::AddFollowerPoseComponent(USkinnedMeshComponent* SkinnedMeshComponent)
{
	FollowerPoseComponents.AddUnique(SkinnedMeshComponent);
}

void USkinnedMeshComponent::RemoveFollowerPoseComponent(USkinnedMeshComponent* SkinnedMeshComponent)
{
	FollowerPoseComponents.Remove(SkinnedMeshComponent);
}

void USkinnedMeshComponent::InvalidateCachedBounds()
{
	bCachedLocalBoundsUpToDate = false;
	bCachedWorldSpaceBoundsUpToDate = false;
	// Also invalidate all follower components.
	for (const TWeakObjectPtr<USkinnedMeshComponent>& SkinnedMeshComp : FollowerPoseComponents)
	{
		if (USkinnedMeshComponent* SkinnedMeshCompPtr = SkinnedMeshComp.Get())
		{
			SkinnedMeshCompPtr->bCachedLocalBoundsUpToDate = false;
			SkinnedMeshCompPtr->bCachedWorldSpaceBoundsUpToDate = false;
		}
	}

	// We need to invalidate all attached skinned mesh components as well
	const TArray<USceneComponent*>& AttachedComps = GetAttachChildren();
	for (USceneComponent* ChildComp : AttachedComps)
	{
		if (USkinnedMeshComponent* SkinnedChild = Cast<USkinnedMeshComponent>(ChildComp))
		{
			if (SkinnedChild->bCachedLocalBoundsUpToDate)
			{
				SkinnedChild->InvalidateCachedBounds();
			}
		}
	}
}

void USkinnedMeshComponent::RefreshFollowerComponents()
{
	for (const TWeakObjectPtr<USkinnedMeshComponent>& MeshComp : FollowerPoseComponents)
	{
		if (USkinnedMeshComponent* MeshCompPtr = MeshComp.Get())
		{
			// Update any children of the follower components if they are using sockets
			MeshCompPtr->UpdateChildTransforms(EUpdateTransformFlags::OnlyUpdateIfUsingSocket);

			MeshCompPtr->MarkRenderDynamicDataDirty();
			MeshCompPtr->MarkRenderTransformDirty();
		}
	}
}

void USkinnedMeshComponent::SetForceWireframe(bool InForceWireframe)
{
	if(bForceWireframe != InForceWireframe)
	{
		bForceWireframe = InForceWireframe;
		MarkRenderStateDirty();
	}
}

UMaterialInterface* USkinnedMeshComponent::GetDefaultOverlayMaterial() const
{
	if (GetSkinnedAsset())
	{
		return GetSkinnedAsset()->GetOverlayMaterial();
	}
	return nullptr;
}

float USkinnedMeshComponent::GetDefaultOverlayMaterialMaxDrawDistance() const
{
	if (GetSkinnedAsset())
	{
		return GetSkinnedAsset()->GetOverlayMaterialMaxDrawDistance();
	}
	return 0.f;
}

void USkinnedMeshComponent::GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const
{
	FSkinnedMeshComponentHelper::GetDefaultMaterialSlotsOverlayMaterial(*this, OutMaterialSlotOverlayMaterials);
}

#if WITH_EDITOR
void USkinnedMeshComponent::SetSectionPreview(int32 InSectionIndexPreview)
{
	if (SectionIndexPreview != InSectionIndexPreview)
	{
		SectionIndexPreview = InSectionIndexPreview;
		MarkRenderStateDirty();
	}
}

void USkinnedMeshComponent::SetMaterialPreview(int32 InMaterialIndexPreview)
{
	if (MaterialIndexPreview != InMaterialIndexPreview)
	{
		MaterialIndexPreview = InMaterialIndexPreview;
		MarkRenderStateDirty();
	}
}

void USkinnedMeshComponent::SetSelectedEditorSection(int32 InSelectedEditorSection)
{
	if (SelectedEditorSection != InSelectedEditorSection)
	{
		SelectedEditorSection = InSelectedEditorSection;
		MarkRenderStateDirty();
	}
}

void USkinnedMeshComponent::SetSelectedEditorMaterial(int32 InSelectedEditorMaterial)
{
	if (SelectedEditorMaterial != InSelectedEditorMaterial)
	{
		SelectedEditorMaterial = InSelectedEditorMaterial;
		MarkRenderStateDirty();
	}
}

#endif // WITH_EDITOR

UMorphTarget* USkinnedMeshComponent::FindMorphTarget( FName MorphTargetName ) const
{
	if(GetSkinnedAsset() != nullptr )
	{
		return GetSkinnedAsset()->FindMorphTarget(MorphTargetName);
	}

	return nullptr;
}

bool USkinnedMeshComponent::GetMissingLeaderBoneRelativeTransform(int32 InBoneIndex, FMissingLeaderBoneCacheEntry& OutInfo) const
{
	const FReferenceSkeleton& FollowerRefSkeleton = GetSkinnedAsset()->GetRefSkeleton();
	check(FollowerRefSkeleton.IsValidIndex(InBoneIndex));
	const TArray<FTransform>& BoneSpaceRefPoseTransforms = FollowerRefSkeleton.GetRefBonePose();

	OutInfo.CommonAncestorBoneIndex = INDEX_NONE;
	OutInfo.RelativeTransform = FTransform::Identity;

	FTransform RelativeTransform = BoneSpaceRefPoseTransforms[InBoneIndex];

	// we need to find a common base component-space transform in this skeletal mesh as it
	// isnt present in the leader, so run up the hierarchy
	int32 CommonAncestorBoneIndex = InBoneIndex;
	while(CommonAncestorBoneIndex != INDEX_NONE)
	{
		CommonAncestorBoneIndex = FollowerRefSkeleton.GetParentIndex(CommonAncestorBoneIndex);
		if(CommonAncestorBoneIndex != INDEX_NONE)
		{
			OutInfo.CommonAncestorBoneIndex = GetLeaderBoneMap()[CommonAncestorBoneIndex];
			if(OutInfo.CommonAncestorBoneIndex != INDEX_NONE)
			{
				OutInfo.RelativeTransform = RelativeTransform;
				return true;
			}

			RelativeTransform = RelativeTransform * BoneSpaceRefPoseTransforms[CommonAncestorBoneIndex];
		}
	}

	return false;
}

void USkinnedMeshComponent::UpdateLeaderBoneMap()
{
	GetMutableLeaderBoneMap().Reset();
	MissingLeaderBoneMap.Reset();

	if (GetSkinnedAsset())
	{
		if (USkinnedMeshComponent* LeaderPoseComponentPtr = LeaderPoseComponent.Get())
		{
			if (USkinnedAsset* LeaderMesh = LeaderPoseComponentPtr->GetSkinnedAsset())
			{
				const FReferenceSkeleton& FollowerRefSkeleton = GetSkinnedAsset()->GetRefSkeleton();
				const FReferenceSkeleton& LeaderRefSkeleton = LeaderMesh->GetRefSkeleton();

				LeaderBoneMap.AddUninitialized(FollowerRefSkeleton.GetNum());
				if (GetSkinnedAsset() == LeaderMesh)
				{
					// if the meshes are the same, the indices must match exactly so we don't need to look them up
					for (int32 BoneIndex = 0; BoneIndex < LeaderBoneMap.Num(); BoneIndex++)
					{
						LeaderBoneMap[BoneIndex] = BoneIndex;
					}
				}
				else
				{
					for (int32 BoneIndex = 0; BoneIndex < LeaderBoneMap.Num(); BoneIndex++)
					{
						const FName BoneName = FollowerRefSkeleton.GetBoneName(BoneIndex);
						LeaderBoneMap[BoneIndex] = LeaderRefSkeleton.FindBoneIndex(BoneName);
					}

					// Cache bones for any SOCKET bones that are missing in the leader.
					// We assume that sockets will be potentially called more often, so we
					// leave out missing BONE transforms here to try to balance memory & performance.
					for(USkeletalMeshSocket* Socket : GetSkinnedAsset()->GetActiveSocketList())
					{
						int32 BoneIndex = FollowerRefSkeleton.FindBoneIndex(Socket->BoneName);
						int32 LeaderBoneIndex = LeaderRefSkeleton.FindBoneIndex(Socket->BoneName);
						if(BoneIndex != INDEX_NONE && LeaderBoneIndex == INDEX_NONE)
						{
							FMissingLeaderBoneCacheEntry MissingBoneInfo;
							if(GetMissingLeaderBoneRelativeTransform(BoneIndex, MissingBoneInfo))
							{
								MissingLeaderBoneMap.Add(BoneIndex, MissingBoneInfo);
							}
						}
					}
				}
			}
		}
	}

	LeaderBoneMapCacheCount += 1;
}

FTransform USkinnedMeshComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	//QUICK_SCOPE_CYCLE_COUNTER(USkinnedMeshComponent_GetSocketTransform);

	FTransform OutSocketTransform = GetComponentTransform();

	if (InSocketName != NAME_None)
	{
		int32 SocketBoneIndex;
		FTransform SocketLocalTransform;
		USkeletalMeshSocket const* const Socket = GetSocketInfoByName(InSocketName, SocketLocalTransform, SocketBoneIndex);
		// apply the socket transform first if we find a matching socket
		if (Socket)
		{
			if (TransformSpace == RTS_ParentBoneSpace)
			{
				//we are done just return now
				return SocketLocalTransform;
			}

			if (SocketBoneIndex != INDEX_NONE)
			{
				FTransform BoneTransform = GetBoneTransform(SocketBoneIndex);
				OutSocketTransform = SocketLocalTransform * BoneTransform;
			}
		}
		else
		{
			int32 BoneIndex = GetBoneIndex(InSocketName);
			if (BoneIndex != INDEX_NONE)
			{
				OutSocketTransform = GetBoneTransform(BoneIndex);

				if (TransformSpace == RTS_ParentBoneSpace)
				{
					FName ParentBone = GetParentBone(InSocketName);
					int32 ParentIndex = GetBoneIndex(ParentBone);
					if (ParentIndex != INDEX_NONE)
					{
						return OutSocketTransform.GetRelativeTransform(GetBoneTransform(ParentIndex));
					}
					return OutSocketTransform.GetRelativeTransform(GetComponentTransform());
				}
			}
		}
	}

	switch(TransformSpace)
	{
		case RTS_Actor:
		{
			if( AActor* Actor = GetOwner() )
			{
				return OutSocketTransform.GetRelativeTransform( Actor->GetTransform() );
			}
			break;
		}
		case RTS_Component:
		{
			return OutSocketTransform.GetRelativeTransform( GetComponentTransform() );
		}
	}

	return OutSocketTransform;
}

class USkeletalMeshSocket const* USkinnedMeshComponent::GetSocketInfoByName(FName InSocketName, FTransform& OutTransform, int32& OutBoneIndex) const
{
	const FName* OverrideSocket = SocketOverrideLookup.Find(InSocketName);
	const FName OverrideSocketName = OverrideSocket ? *OverrideSocket : InSocketName;

	USkeletalMeshSocket const* Socket = nullptr;

	if(GetSkinnedAsset())
	{
		int32 SocketIndex;
		Socket = GetSkinnedAsset()->FindSocketInfo(OverrideSocketName, OutTransform, OutBoneIndex, SocketIndex);
	}
	else
	{
		if (OverrideSocket)
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "GetSocketInfoByName(%ls -> override To %ls): No SkeletalMesh for Component(%ls) Actor(%ls)",
				*InSocketName.ToString(), *OverrideSocketName.ToString(), *GetName(), *GetFullNameSafe(GetOuter()));
		}
		else
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "GetSocketInfoByName(%ls): No SkeletalMesh for Component(%ls) Actor(%ls)",
				*OverrideSocketName.ToString(), *GetName(), *GetFullNameSafe(GetOuter()));
		}
	}

	return Socket;
}

class USkeletalMeshSocket const* USkinnedMeshComponent::GetSocketByName(FName InSocketName) const
{
	const FName* OverrideSocket = SocketOverrideLookup.Find(InSocketName);
	const FName OverrideSocketName = OverrideSocket ? *OverrideSocket : InSocketName;

	USkeletalMeshSocket const* Socket = nullptr;

	if(GetSkinnedAsset())
	{
		Socket = GetSkinnedAsset()->FindSocket(OverrideSocketName);
	}
	else
	{
		if (OverrideSocket)
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "GetSocketByName(%ls -> override To %ls): No SkeletalMesh for Component(%ls) Actor(%ls)",
				*InSocketName.ToString(), *OverrideSocketName.ToString(), *GetName(), *GetFullNameSafe(GetOuter()));
		}
		else
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "GetSocketByName(%ls): No SkeletalMesh for Component(%ls) Actor(%ls)",
				*OverrideSocketName.ToString(), *GetName(), *GetFullNameSafe(GetOuter()));
		}
	}

	return Socket;
}

void USkinnedMeshComponent::AddSocketOverride(FName SourceSocketName, FName OverrideSocketName, bool bWarnHasOverrided)
{
	if (FName* FoundName = SocketOverrideLookup.Find(SourceSocketName))
	{
		if (*FoundName != OverrideSocketName)
		{
			if (bWarnHasOverrided)
			{
				UE_LOGF(LogSkinnedMeshComp, Warning, "AddSocketOverride(%ls, %ls): Component(%ls) Actor(%ls) has already defined an override for socket(%ls), replacing %ls as override",
					*SourceSocketName.ToString(), *OverrideSocketName.ToString(), *GetName(), *GetNameSafe(GetOuter()), *SourceSocketName.ToString(), *(FoundName->ToString()));
			}
			*FoundName = OverrideSocketName;
		}
	}
	else
	{
		SocketOverrideLookup.Add(SourceSocketName, OverrideSocketName);
	}
}

void USkinnedMeshComponent::RemoveSocketOverrides(FName SourceSocketName)
{
	SocketOverrideLookup.Remove(SourceSocketName);
}

void USkinnedMeshComponent::RemoveAllSocketOverrides()
{
	SocketOverrideLookup.Reset();
}

void USkinnedMeshComponent::OnChildAttached(USceneComponent* ChildComponent)
{
	Super::OnChildAttached(ChildComponent);
	UpdateCachedSocketAttachmentFlag();
}

void USkinnedMeshComponent::OnChildDetached(USceneComponent* ChildComponent)
{
	Super::OnChildDetached(ChildComponent);
	UpdateCachedSocketAttachmentFlag();
}

void USkinnedMeshComponent::UpdateCachedSocketAttachmentFlag()
{
	// Update whether any of our attachments are socket-attached
	bHasSocketAttachments = false;
	for (const USceneComponent* AttachedComponent : GetAttachChildren())
	{
		if (AttachedComponent && AttachedComponent->GetAttachSocketName() != NAME_None)
		{
			bHasSocketAttachments = true;
			break;
		}
	}
}

bool USkinnedMeshComponent::HasAnyAttachedSockets() const
{
	return bHasSocketAttachments;
}

bool USkinnedMeshComponent::DoesSocketExist(FName InSocketName) const
{
	return (GetSocketBoneName(InSocketName) != NAME_None);
}

FName USkinnedMeshComponent::GetSocketBoneName(FName InSocketName) const
{
	if(!GetSkinnedAsset())
	{
		return NAME_None;
	}

	const FName* OverrideSocket = SocketOverrideLookup.Find(InSocketName);
	const FName OverrideSocketName = OverrideSocket ? *OverrideSocket : InSocketName;

	// First check for a socket
	USkeletalMeshSocket const* TmpSocket = GetSkinnedAsset()->FindSocket(OverrideSocketName);
	if( TmpSocket )
	{
		return TmpSocket->BoneName;
	}

	// If socket is not found, maybe it was just a bone name.
	if( GetBoneIndex(OverrideSocketName) != INDEX_NONE )
	{
		return OverrideSocketName;
	}

	// Doesn't exist.
	return NAME_None;
}


FQuat USkinnedMeshComponent::GetBoneQuaternion(FName BoneName, EBoneSpaces::Type Space) const
{
	int32 BoneIndex = GetBoneIndex(BoneName);

	if( BoneIndex == INDEX_NONE )
	{
		UE_LOGF(LogSkinnedMeshComp, Warning, "USkinnedMeshComponent::execGetBoneQuaternion : Could not find bone: %ls", *BoneName.ToString());
		return FQuat::Identity;
	}

	FTransform BoneTransform;
	if( Space == EBoneSpaces::ComponentSpace )
	{
		const USkinnedMeshComponent* const LeaderPoseComponentInst = LeaderPoseComponent.Get();
		if(LeaderPoseComponentInst)
		{
			if(BoneIndex < GetLeaderBoneMap().Num())
			{
				int32 ParentBoneIndex = GetLeaderBoneMap()[BoneIndex];
				// If ParentBoneIndex is valid, grab matrix from LeaderPoseComponent.
				if(	ParentBoneIndex != INDEX_NONE && 
					ParentBoneIndex < LeaderPoseComponentInst->GetNumComponentSpaceTransforms())
				{
					BoneTransform = LeaderPoseComponentInst->GetComponentSpaceTransforms()[ParentBoneIndex];
				}
				else
				{
					BoneTransform = FTransform::Identity;
				}
			}
			else
			{
				BoneTransform = FTransform::Identity;
			}
		}
		else
		{
			BoneTransform = GetComponentSpaceTransforms()[BoneIndex];
		}
	}
	else
	{
		BoneTransform = GetBoneTransform(BoneIndex);
	}

	BoneTransform.RemoveScaling();
	return BoneTransform.GetRotation();
}


FVector USkinnedMeshComponent::GetBoneLocation(FName BoneName, EBoneSpaces::Type Space) const
{
	int32 BoneIndex = GetBoneIndex(BoneName);
	if( BoneIndex == INDEX_NONE )
	{
		UE_LOGF(LogAnimation, Log, "USkinnedMeshComponent::GetBoneLocation (%ls %ls): Could not find bone: %ls", *GetFullName(), *GetDetailedInfo(), *BoneName.ToString() );
		return FVector::ZeroVector;
	}

	switch (Space)
	{
	case EBoneSpaces::ComponentSpace:
	{
		const USkinnedMeshComponent* const LeaderPoseComponentInst = LeaderPoseComponent.Get();
		if(LeaderPoseComponentInst)
		{
			if(BoneIndex < GetLeaderBoneMap().Num())
			{
				int32 ParentBoneIndex = GetLeaderBoneMap()[BoneIndex];
				// If ParentBoneIndex is valid, grab transform from LeaderPoseComponent.
				if(	ParentBoneIndex != INDEX_NONE && 
					ParentBoneIndex < LeaderPoseComponentInst->GetNumComponentSpaceTransforms())
				{
					return LeaderPoseComponentInst->GetComponentSpaceTransforms()[ParentBoneIndex].GetLocation();
				}
			}

			// return empty vector
			return FVector::ZeroVector;
		}
		else
		{
			return GetComponentSpaceTransforms()[BoneIndex].GetLocation();
		}
	}

	case EBoneSpaces::WorldSpace:
		// To support non-uniform scale (via LocalToWorld), use GetBoneMatrix
		return GetBoneMatrix(BoneIndex).GetOrigin();

	default:
		check(false); // Unknown BoneSpace
		return FVector::ZeroVector;
	}
}


FVector USkinnedMeshComponent::GetBoneAxis( FName BoneName, EAxis::Type Axis ) const
{
	int32 BoneIndex = GetBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogSkinnedMeshComp, Warning, "USkinnedMeshComponent::execGetBoneAxis : Could not find bone: %ls", *BoneName.ToString());
		return FVector::ZeroVector;
	}
	else if (Axis == EAxis::None)
	{
		UE_LOGF(LogSkinnedMeshComp, Warning, "USkinnedMeshComponent::execGetBoneAxis: Invalid axis specified");
		return FVector::ZeroVector;
	}
	else
	{
		return GetBoneMatrix(BoneIndex).GetUnitAxis(Axis);
	}
}

bool USkinnedMeshComponent::HasAnySockets() const
{
	return (GetSkinnedAsset() != nullptr) && (
#if WITH_EDITOR
		(GetSkinnedAsset()->GetActiveSocketList().Num() > 0) ||
#endif
		(GetSkinnedAsset()->GetRefSkeleton().GetNum() > 0));
}

void USkinnedMeshComponent::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const
{
	if (GetSkinnedAsset() != nullptr)
	{
		// Grab all the mesh and skeleton sockets
		const TArray<USkeletalMeshSocket*> AllSockets = GetSkinnedAsset()->GetActiveSocketList();

		for (int32 SocketIdx = 0; SocketIdx < AllSockets.Num(); ++SocketIdx)
		{
			if (USkeletalMeshSocket* Socket = AllSockets[SocketIdx])
			{
				new (OutSockets) FComponentSocketDescription(Socket->SocketName, EComponentSocketType::Socket);
			}
		}

		// Now grab the bones, which can behave exactly like sockets
		for (int32 BoneIdx = 0; BoneIdx < GetSkinnedAsset()->GetRefSkeleton().GetNum(); ++BoneIdx)
		{
			const FName BoneName = GetSkinnedAsset()->GetRefSkeleton().GetBoneName(BoneIdx);
			new (OutSockets) FComponentSocketDescription(BoneName, EComponentSocketType::Bone);
		}
	}
}

bool USkinnedMeshComponent::UpdateOverlapsImpl(const TOverlapArrayView* PendingOverlaps, bool bDoNotifies, const TOverlapArrayView* OverlapsAtEndLocation)
{
	// we don't support overlap test on destructible or physics asset
	// so use SceneComponent::UpdateOverlaps to handle children
	return USceneComponent::UpdateOverlapsImpl(PendingOverlaps, bDoNotifies, OverlapsAtEndLocation);
}

#if WITH_EDITOR
bool USkinnedMeshComponent::GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty)
{
	if(OverrideMaterials.IsValidIndex(ElementIndex))
	{
		OutOwner = this;
		OutPropertyPath = FString::Printf(TEXT("%s[%d]"), GET_MEMBER_NAME_STRING_CHECKED(UMeshComponent, OverrideMaterials), ElementIndex);
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(UMeshComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials))))
		{
			OutProperty = ArrayProperty->Inner;
		}
		return true;
	}
	if (GetSkinnedAsset() && GetSkinnedAsset()->GetMaterials().IsValidIndex(ElementIndex))
	{
		OutOwner = GetSkinnedAsset();
		OutPropertyPath = FString::Printf(TEXT("%s[%d].%s"), *USkeletalMesh::GetMaterialsMemberName().ToString(), ElementIndex, GET_MEMBER_NAME_STRING_CHECKED(FSkeletalMaterial, MaterialInterface));
		OutProperty = FSkeletalMaterial::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSkeletalMaterial, MaterialInterface));
		return true;
	}

	return false;
}
#endif // WITH_EDITOR

void USkinnedMeshComponent::TransformToBoneSpace(FName BoneName, FVector InPosition, FRotator InRotation, FVector& OutPosition, FRotator& OutRotation) const
{
	int32 BoneIndex = GetBoneIndex(BoneName);
	if(BoneIndex != INDEX_NONE)
	{
		FMatrix BoneToWorldTM = GetBoneMatrix(BoneIndex);
		FMatrix WorldTM = FRotationTranslationMatrix(InRotation, InPosition);
		FMatrix LocalTM = WorldTM * BoneToWorldTM.Inverse();

		OutPosition = LocalTM.GetOrigin();
		OutRotation = LocalTM.Rotator();
	}
}


void USkinnedMeshComponent::TransformFromBoneSpace(FName BoneName, FVector InPosition, FRotator InRotation, FVector& OutPosition, FRotator& OutRotation)
{
	int32 BoneIndex = GetBoneIndex(BoneName);
	if(BoneIndex != INDEX_NONE)
	{
		FMatrix BoneToWorldTM = GetBoneMatrix(BoneIndex);

		FMatrix LocalTM = FRotationTranslationMatrix(InRotation, InPosition);
		FMatrix WorldTM = LocalTM * BoneToWorldTM;

		OutPosition = WorldTM.GetOrigin();
		OutRotation = WorldTM.Rotator();
	}
}



FName USkinnedMeshComponent::FindClosestBone(FVector TestLocation, FVector* BoneLocation, float IgnoreScale, bool bRequirePhysicsAsset) const
{
	if (GetSkinnedAsset() == nullptr)
	{
		if (BoneLocation != nullptr)
		{
			*BoneLocation = FVector::ZeroVector;
		}
		return NAME_None;
	}
	else
	{
		// cache the physics asset
		const UPhysicsAsset* PhysAsset = GetPhysicsAsset();
		if (bRequirePhysicsAsset && !PhysAsset)
		{
			if (BoneLocation != nullptr)
			{
				*BoneLocation = FVector::ZeroVector;
			}
			return NAME_None;
		}

		// transform the TestLocation into mesh local space so we don't have to transform the (mesh local) bone locations
		TestLocation = GetComponentTransform().InverseTransformPosition(TestLocation);
		
		float IgnoreScaleSquared = FMath::Square(IgnoreScale);
		float BestDistSquared = UE_BIG_NUMBER;
		int32 BestIndex = -1;

		const USkinnedMeshComponent* BaseComponent = LeaderPoseComponent.IsValid() ? LeaderPoseComponent.Get() : this;
		const TArray<FTransform>& CompSpaceTransforms = BaseComponent->GetComponentSpaceTransforms();

		for (int32 i = 0; i < BaseComponent->GetNumComponentSpaceTransforms(); i++)
		{
			// If we require a physics asset, then look it up in the map
			bool bPassPACheck = !bRequirePhysicsAsset;
			if (bRequirePhysicsAsset)
			{
				FName BoneName = GetSkinnedAsset()->GetRefSkeleton().GetBoneName(i);
				bPassPACheck = (PhysAsset->BodySetupIndexMap.Find(BoneName) != nullptr);
			}

			if (bPassPACheck && (IgnoreScale < 0.f || CompSpaceTransforms[i].GetScaledAxis(EAxis::X).SizeSquared() > IgnoreScaleSquared))
			{
				float DistSquared = (TestLocation - CompSpaceTransforms[i].GetLocation()).SizeSquared();
				if (DistSquared < BestDistSquared)
				{
					BestIndex = i;
					BestDistSquared = DistSquared;
				}
			}
		}

		if (BestIndex == -1)
		{
			if (BoneLocation != nullptr)
			{
				*BoneLocation = FVector::ZeroVector;
			}
			return NAME_None;
		}
		else
		{
			// transform the bone location into world space
			if (BoneLocation != nullptr)
			{
				*BoneLocation = (CompSpaceTransforms[BestIndex] * GetComponentTransform()).GetLocation();
			}
			return GetSkinnedAsset()->GetRefSkeleton().GetBoneName(BestIndex);
		}
	}
}

FName USkinnedMeshComponent::FindClosestBone_K2(FVector TestLocation, FVector& BoneLocation, float IgnoreScale, bool bRequirePhysicsAsset) const
{
	BoneLocation = FVector::ZeroVector;
	return FindClosestBone(TestLocation, &BoneLocation, IgnoreScale, bRequirePhysicsAsset);
}

void USkinnedMeshComponent::ShowMaterialSection(int32 MaterialID, int32 SectionIndex, bool bShow, int32 LODIndex)
{
	if (!GetSkinnedAsset())
	{
		// no skeletalmesh, then nothing to do. 
		return;
	}
	// Make sure LOD info for this component has been initialized
	InitLODInfos();
	if (LODInfo.IsValidIndex(LODIndex))
	{
		const FSkeletalMeshLODInfo& SkelLODInfo = *GetSkinnedAsset()->GetLODInfo(LODIndex);
		FSkelMeshComponentLODInfo& SkelCompLODInfo = LODInfo[LODIndex];
		TArray<bool>& HiddenMaterials = SkelCompLODInfo.HiddenMaterials;
	
		// allocate if not allocated yet
		if ( HiddenMaterials.Num() != GetSkinnedAsset()->GetMaterials().Num() )
		{
			// Using skeletalmesh component because Materials.Num() should be <= GetSkinnedAsset()->GetMaterials().Num()		
			HiddenMaterials.Empty(GetSkinnedAsset()->GetMaterials().Num());
			HiddenMaterials.AddZeroed(GetSkinnedAsset()->GetMaterials().Num());
		}
		// If we have a valid LODInfo LODMaterialMap, route material index through it.
		int32 UseMaterialIndex = MaterialID;			
		if(SkelLODInfo.LODMaterialMap.IsValidIndex(SectionIndex) && SkelLODInfo.LODMaterialMap[SectionIndex] != INDEX_NONE)
		{
			UseMaterialIndex = SkelLODInfo.LODMaterialMap[SectionIndex];
			UseMaterialIndex = FMath::Clamp( UseMaterialIndex, 0, HiddenMaterials.Num() );
		}
		// Mark the mapped section material entry as visible/hidden
		if (HiddenMaterials.IsValidIndex(UseMaterialIndex))
		{
			HiddenMaterials[UseMaterialIndex] = !bShow;
		}

		if ( MeshObject )
		{
			// need to send render thread for updated hidden section
			FSkeletalMeshObject* InMeshObject = MeshObject;
			ENQUEUE_RENDER_COMMAND(FUpdateHiddenSectionCommand)(
				[InMeshObject, HiddenMaterials, LODIndex](FRHICommandListImmediate& RHICmdList)
			{
				InMeshObject->SetHiddenMaterials(LODIndex, HiddenMaterials);
			});
		}

		MarkRenderStateDirty();
	}
}

void USkinnedMeshComponent::ShowAllMaterialSections(int32 LODIndex)
{
	InitLODInfos();
	if (LODInfo.IsValidIndex(LODIndex))
	{
		FSkelMeshComponentLODInfo& SkelCompLODInfo = LODInfo[LODIndex];
		TArray<bool>& HiddenMaterials = SkelCompLODInfo.HiddenMaterials;

		// Only need to do anything if array is allocated - otherwise nothing is being hidden
		if (HiddenMaterials.Num() > 0)
		{
			for (int32 MatIdx = 0; MatIdx < HiddenMaterials.Num(); MatIdx++)
			{
				HiddenMaterials[MatIdx] = false;
			}

			if (MeshObject)
			{
				// need to send render thread for updated hidden section
				FSkeletalMeshObject* InMeshObject = MeshObject;
				ENQUEUE_RENDER_COMMAND(FUpdateHiddenSectionCommand)(
					[InMeshObject, HiddenMaterials, LODIndex](FRHICommandListImmediate& RHICmdList)
					{
						InMeshObject->SetHiddenMaterials(LODIndex, HiddenMaterials);
					});
			}
			MarkRenderStateDirty();
		}
	}
}

bool USkinnedMeshComponent::IsMaterialSectionShown(int32 MaterialID, int32 LODIndex)
{
	bool bHidden = false;
	if (LODInfo.IsValidIndex(LODIndex))
	{
		FSkelMeshComponentLODInfo& SkelCompLODInfo = LODInfo[LODIndex];
		TArray<bool>& HiddenMaterials = SkelCompLODInfo.HiddenMaterials;
		if (HiddenMaterials.IsValidIndex(MaterialID))
		{
			bHidden = HiddenMaterials[MaterialID];
		}
	}
	return !bHidden;
}


void USkinnedMeshComponent::GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials ) const
{
	FSkinnedMeshComponentHelper::GetUsedMaterials(*this, OutMaterials, bGetDebugMaterials);
}

FSkinWeightVertexBuffer* USkinnedMeshComponent::GetSkinWeightBuffer(int32 LODIndex) const
{
	FSkinWeightVertexBuffer* WeightBuffer = nullptr;

	if (GetSkinnedAsset() &&
		GetSkinnedAsset()->GetResourceForRendering() && 
		GetSkinnedAsset()->GetResourceForRendering()->LODRenderData.IsValidIndex(LODIndex) )
	{
		FSkeletalMeshLODRenderData& LODData = GetSkinnedAsset()->GetResourceForRendering()->LODRenderData[LODIndex];

		// Grab weight buffer (check for override)
		if (LODInfo.IsValidIndex(LODIndex) &&
			LODInfo[LODIndex].OverrideSkinWeights && 
			LODInfo[LODIndex].OverrideSkinWeights->GetNumVertices() == LODData.GetNumVertices())
		{
			WeightBuffer = LODInfo[LODIndex].OverrideSkinWeights;
		}
		else if (LODInfo.IsValidIndex(LODIndex) &&
			LODInfo[LODIndex].OverrideProfileSkinWeights &&
			LODInfo[LODIndex].OverrideProfileSkinWeights->GetNumVertices() == LODData.GetNumVertices())
		{
			WeightBuffer = LODInfo[LODIndex].OverrideProfileSkinWeights;
		}
		else
		{
			WeightBuffer = LODData.GetSkinWeightVertexBuffer();
		}
	}

	return WeightBuffer;
}

FVector3f USkinnedMeshComponent::GetSkinnedVertexPosition(USkinnedMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& LODData, FSkinWeightVertexBuffer& SkinWeightBuffer) 
{
	FVector SkinnedPos(0, 0, 0);

	int32 SectionIndex;
	int32 VertIndex;
	LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, VertIndex);

	check(SectionIndex < LODData.RenderSections.Num());
	const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];

	return GetTypedSkinnedVertexPosition<false>(Component, Section, LODData.StaticVertexBuffers.PositionVertexBuffer, SkinWeightBuffer, VertIndex);
}

FVector3f USkinnedMeshComponent::GetSkinnedVertexPosition(USkinnedMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& LODData, FSkinWeightVertexBuffer& SkinWeightBuffer, TArray<FMatrix44f>& CachedRefToLocals) 
{
	FVector SkinnedPos(0, 0, 0);
	
	int32 SectionIndex;
	int32 VertIndex;
	LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, VertIndex);

	check(SectionIndex < LODData.RenderSections.Num());
	const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];

	return GetTypedSkinnedVertexPosition<false>(Component, Section, LODData.StaticVertexBuffers.PositionVertexBuffer, SkinWeightBuffer, VertIndex, CachedRefToLocals);
}

void USkinnedMeshComponent::SetRefPoseOverride(const TArray<FTransform>& NewRefPoseTransforms)
{
	if (!GetSkinnedAsset())
	{
		UE_LOGF(LogSkeletalMesh, Warning, "SetRefPoseOverride (%ls) : Not valid without SkeletalMesh assigned.", *GetName());
		return;
	}

	const int32 NumRealBones = GetSkinnedAsset()->GetRefSkeleton().GetRawBoneNum();

	if (NumRealBones != NewRefPoseTransforms.Num())
	{
		UE_LOGF(LogSkeletalMesh, Warning, "SetRefPoseOverride (%ls) : Expected %d transforms, got %d.", *GetSkinnedAsset()->GetName(), NumRealBones, NewRefPoseTransforms.Num());
		return;
	}

	// Always allocate new struct to keep info.
	// previously allocated RefPoseOverride, if there was one, will potentially be used on other threads by BoneContainer for one frame
	RefPoseOverride = MakeShared<FSkelMeshRefPoseOverride>();

	const int32 NumBones = GetSkinnedAsset()->GetRefSkeleton().GetNum();

	// Copy input transforms into override data
	RefPoseOverride->RefBonePoses.Reserve(NumBones);
	RefPoseOverride->RefBonePoses = NewRefPoseTransforms;

	// Add additional space for any virtual bones & copy them in
	if(NumBones > NumRealBones)
	{
		RefPoseOverride->RefBonePoses.SetNumUninitialized(NumBones);
		const TArray<FTransform>& RefPose = GetSkinnedAsset()->GetRefSkeleton().GetRefBonePose();
		FMemory::Memcpy(&RefPoseOverride->RefBonePoses[NumRealBones], &RefPose[NumRealBones], sizeof(FTransform) * (NumBones - NumRealBones));
	}

	// Allocate output inv matrices
	RefPoseOverride->RefBasesInvMatrix.AddUninitialized(NumRealBones);

	// Reset cached mesh-space ref pose
	TArray<FMatrix44f> CachedComposedRefPoseMatrices;
	CachedComposedRefPoseMatrices.AddUninitialized(NumRealBones);

	// Compute the RefBasesInvMatrix array
	for (int32 BoneIndex = 0; BoneIndex < NumRealBones; BoneIndex++)
	{
		FTransform BoneTransform = RefPoseOverride->RefBonePoses[BoneIndex];
		// Make sure quaternion is normalized!
		BoneTransform.NormalizeRotation();

		// Render the default pose.
		CachedComposedRefPoseMatrices[BoneIndex] = FMatrix44f(BoneTransform.ToMatrixWithScale());

		// Construct mesh-space skeletal hierarchy.
		if (BoneIndex > 0)
		{
			int32 ParentIndex = GetSkinnedAsset()->GetRefSkeleton().GetRawParentIndex(BoneIndex);
			CachedComposedRefPoseMatrices[BoneIndex] = CachedComposedRefPoseMatrices[BoneIndex] * CachedComposedRefPoseMatrices[ParentIndex];
		}

		// Check for zero matrix
		FVector3f XAxis, YAxis, ZAxis;
		CachedComposedRefPoseMatrices[BoneIndex].GetScaledAxes(XAxis, YAxis, ZAxis);
		if (XAxis.IsNearlyZero(UE_SMALL_NUMBER) &&
			YAxis.IsNearlyZero(UE_SMALL_NUMBER) &&
			ZAxis.IsNearlyZero(UE_SMALL_NUMBER))
		{
			// this is not allowed, warn them 
			UE_LOGF(LogSkeletalMesh, Warning, "Reference Pose for asset %ls for joint (%ls) includes NIL matrix. Zero scale isn't allowed on ref pose. ", *GetSkinnedAsset()->GetPathName(), *GetSkinnedAsset()->GetRefSkeleton().GetBoneName(BoneIndex).ToString());
		}

		// Precompute inverse so we can use from-refpose-skin vertices.
		RefPoseOverride->RefBasesInvMatrix[BoneIndex] = CachedComposedRefPoseMatrices[BoneIndex].Inverse();
	}
}

void USkinnedMeshComponent::ClearRefPoseOverride()
{
	// Release mem for override info
	if (RefPoseOverride)
	{
		RefPoseOverride = nullptr;
	}
}

void USkinnedMeshComponent::CacheRefToLocalMatrices(TArray<FMatrix44f>& OutRefToLocal)const
{
	const USkinnedMeshComponent* BaseComponent = GetBaseComponent();
	OutRefToLocal.SetNumUninitialized(GetSkinnedAsset()->GetRefBasesInvMatrix().Num());
	const TArray<FTransform>& CompSpaceTransforms = BaseComponent->GetComponentSpaceTransforms();
	if(CompSpaceTransforms.Num())
	{
		check(CompSpaceTransforms.Num() >= OutRefToLocal.Num());

		for (int32 MatrixIdx = 0; MatrixIdx < OutRefToLocal.Num(); ++MatrixIdx)
		{
			OutRefToLocal[MatrixIdx] = GetSkinnedAsset()->GetRefBasesInvMatrix()[MatrixIdx] * FMatrix44f(CompSpaceTransforms[MatrixIdx].ToMatrixWithScale());
		}
	}
	else
	{
		//Possible in some cases to request this before the component space transforms are prepared (undo/redo)
		for (int32 MatrixIdx = 0; MatrixIdx < OutRefToLocal.Num(); ++MatrixIdx)
		{
			OutRefToLocal[MatrixIdx] = GetSkinnedAsset()->GetRefBasesInvMatrix()[MatrixIdx];
		}
	}
}

void USkinnedMeshComponent::GetCurrentRefToLocalMatrices(TArray<FMatrix44f>& OutRefToLocals, int32 InLodIdx, const TArray<FBoneIndexType>* ExtraRequiredBoneIndices) const
{
	if (const USkinnedAsset* const Asset = GetSkinnedAsset())
	{
		const FSkeletalMeshRenderData* const RenderData = Asset->GetResourceForRendering();
		
		if (ensureMsgf(RenderData->LODRenderData.IsValidIndex(InLodIdx),
			TEXT("GetCurrentRefToLocalMatrices (SkelMesh :%s) input LODIndex (%d) doesn't match with render data size (%d)."),
			*Asset->GetPathName(), InLodIdx, RenderData->LODRenderData.Num()))
		{
			UpdateRefToLocalMatrices(OutRefToLocals, FSkinnedMeshSceneProxyDynamicData(this), GetSkinnedAsset(), RenderData, InLodIdx, ExtraRequiredBoneIndices);
		}
		else
		{
			const FReferenceSkeleton& RefSkeleton = Asset->GetRefSkeleton();
			OutRefToLocals.AddUninitialized(RefSkeleton.GetNum());
			for (int32 Index = 0; Index < OutRefToLocals.Num(); ++Index)
			{
				OutRefToLocals[Index] = FMatrix44f::Identity;
			}
		}
	}
}

void USkinnedMeshComponent::ComputeSkinnedPositions(USkinnedMeshComponent* Component, TArray<FVector3f> & OutPositions, TArray<FMatrix44f>& CachedRefToLocals, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightBuffer)
{
	OutPositions.Empty();

	// Fail if no mesh
	if (!Component || !Component->GetSkinnedAsset())
	{
		return;
	}
	OutPositions.AddUninitialized(LODData.GetNumVertices());

	//update positions
	for (int32 SectionIdx = 0; SectionIdx < LODData.RenderSections.Num(); ++SectionIdx)
	{
		const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIdx];
		{
			//soft
			const uint32 SoftOffset = Section.BaseVertexIndex;
			const uint32 NumSoftVerts = Section.NumVertices;
			for (uint32 SoftIdx = 0; SoftIdx < NumSoftVerts; ++SoftIdx)
			{
				FVector3f SkinnedPosition = GetTypedSkinnedVertexPosition<true>(Component, Section, LODData.StaticVertexBuffers.PositionVertexBuffer, SkinWeightBuffer, SoftIdx, CachedRefToLocals);
				OutPositions[SoftOffset + SoftIdx] = SkinnedPosition;
			}
		}
	}
}

FColor USkinnedMeshComponent::GetVertexColor(int32 VertexIndex) const
{
	// Fail if no mesh or no color vertex buffer.
	FColor FallbackColor = FColor(255, 255, 255, 255);
	if (!GetSkinnedAsset() || !MeshObject)
	{
		return FallbackColor;
	}

	// If there is an override, return that
	if (LODInfo.Num() > 0 && 
		LODInfo[0].OverrideVertexColors != nullptr && 
		LODInfo[0].OverrideVertexColors->IsInitialized() &&
		VertexIndex < (int32)LODInfo[0].OverrideVertexColors->GetNumVertices() )
	{
		return LODInfo[0].OverrideVertexColors->VertexColor(VertexIndex);
	}

	FSkeletalMeshLODRenderData& LODData = MeshObject->GetSkeletalMeshRenderData().LODRenderData[0];
	
	if (!LODData.StaticVertexBuffers.ColorVertexBuffer.IsInitialized())
	{
		return FallbackColor;
	}

	// Find the chunk and vertex within that chunk, and skinning type, for this vertex.
	int32 SectionIndex;
	int32 VertIndex;
	LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, VertIndex);

	check(SectionIndex < LODData.RenderSections.Num());
	const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];
	
	int32 VertexBase = Section.BaseVertexIndex;

	return LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexBase + VertIndex);
}

FVector2D USkinnedMeshComponent::GetVertexUV(int32 VertexIndex, uint32 UVChannel) const
{
	// Fail if no mesh or no vertex buffer.
	FVector2D FallbackUV = FVector2D::ZeroVector;
	if (!GetSkinnedAsset() || !MeshObject)
	{
		return FallbackUV;
	}

	FSkeletalMeshLODRenderData& LODData = MeshObject->GetSkeletalMeshRenderData().LODRenderData[0];
	
	if (!LODData.StaticVertexBuffers.StaticMeshVertexBuffer.IsInitialized())
	{
		return FallbackUV;
	}

	// Find the chunk and vertex within that chunk, and skinning type, for this vertex.
	int32 SectionIndex;
	int32 VertIndex;
	LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, VertIndex);

	check(SectionIndex < LODData.RenderSections.Num());
	const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];
	
	int32 VertexBase = Section.BaseVertexIndex;
	uint32 ClampedUVChannel = FMath::Min(UVChannel, LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords());

	return FVector2D(LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexBase + VertIndex, ClampedUVChannel));
}

void USkinnedMeshComponent::HideBone( int32 BoneIndex, EPhysBodyOp PhysBodyOption)
{
	TArray<uint8>& EditableBoneVisibilityStates = GetEditableBoneVisibilityStates();
	if (ShouldUpdateBoneVisibility() && BoneIndex < EditableBoneVisibilityStates.Num())
	{
		checkSlow ( BoneIndex != INDEX_NONE );
		EditableBoneVisibilityStates[ BoneIndex ] = BVS_ExplicitlyHidden;
		RebuildVisibilityArray();
	}
}


void USkinnedMeshComponent::UnHideBone( int32 BoneIndex )
{
	TArray<uint8>& EditableBoneVisibilityStates = GetEditableBoneVisibilityStates();
	if (ShouldUpdateBoneVisibility() && BoneIndex < EditableBoneVisibilityStates.Num())
	{
		checkSlow ( BoneIndex != INDEX_NONE );
		//@TODO: If unhiding the child of a still hidden bone (coming in, BoneVisibilityStates(RefSkel(BoneIndex).ParentIndex) != BVS_Visible),
		// should we be re-enabling collision bodies?
		// Setting visible to true here is OK in either case as it will be reset to BVS_HiddenByParent in RecalcRequiredBones later if needed.
		EditableBoneVisibilityStates[ BoneIndex ] = BVS_Visible;
		RebuildVisibilityArray();
	}
}


bool USkinnedMeshComponent::IsBoneHidden( int32 BoneIndex ) const
{
	const TArray<uint8>& EditableBoneVisibilityStates = GetEditableBoneVisibilityStates();
	if (ShouldUpdateBoneVisibility() && BoneIndex < EditableBoneVisibilityStates.Num())
	{
		if ( BoneIndex != INDEX_NONE )
		{
			return EditableBoneVisibilityStates[ BoneIndex ] != BVS_Visible;
		}
	}
	else if (USkinnedMeshComponent* LeaderPoseComponentPtr = LeaderPoseComponent.Get())
	{
		return LeaderPoseComponentPtr->IsBoneHidden( BoneIndex );
	}

	return false;
}


bool USkinnedMeshComponent::IsBoneHiddenByName( FName BoneName )
{
	// Find appropriate BoneIndex
	int32 BoneIndex = GetBoneIndex(BoneName);
	if(BoneIndex != INDEX_NONE)
	{
		return IsBoneHidden(BoneIndex);
	}

	return false;
}

void USkinnedMeshComponent::HideBoneByName( FName BoneName, EPhysBodyOp PhysBodyOption )
{
	// Find appropriate BoneIndex
	int32 BoneIndex = GetBoneIndex(BoneName);
	if ( BoneIndex != INDEX_NONE )
	{
		HideBone(BoneIndex, PhysBodyOption);
	}
}


void USkinnedMeshComponent::UnHideBoneByName( FName BoneName )
{
	int32 BoneIndex = GetBoneIndex(BoneName);
	if ( BoneIndex != INDEX_NONE )
	{
		UnHideBone(BoneIndex);
	}
}

void USkinnedMeshComponent::SetForcedLOD(int32 InNewForcedLOD)
{
	SetForceStreamedLOD(InNewForcedLOD - 1);
	SetForceRenderedLOD(InNewForcedLOD - 1);
}

int32 USkinnedMeshComponent::GetForcedLOD() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if !UE_BUILD_SHIPPING
	return GSkeletalMeshForceLOD < 0 ? ForcedLodModel : GSkeletalMeshForceLOD + 1; // Add 1 to CVar because requires the return value is 1-based for legacy reasons.
#else
	return ForcedLodModel;
#endif

PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int32 USkinnedMeshComponent::GetNumLODs() const
{
	return FSkinnedMeshComponentHelper::GetNumLODs(*this);
}

void USkinnedMeshComponent::SetMinLOD(int32 InNewMinLOD)
{
	MinLodModel = GetValidMinLOD(InNewMinLOD);
	MarkRenderStateDirty();
}

void USkinnedMeshComponent::OverrideMinLOD(int32 InNewMinLOD)
{
	MinLodModel = GetValidMinLOD(InNewMinLOD);
	bOverrideMinLod = true;
	MarkRenderStateDirty();
}

int32 USkinnedMeshComponent::ComputeMinLOD() const
{
	return FSkinnedMeshComponentHelper::ComputeMinLOD(*this);
}

int32 USkinnedMeshComponent::GetValidMinLOD(const int32 InMinLODIndex) const
{
	return FSkinnedMeshComponentHelper::GetValidMinLOD(*this, InMinLODIndex);
}

#if WITH_EDITOR
int32 USkinnedMeshComponent::GetLODBias() const
{
	return GSkeletalMeshLODBias;
}
#endif

void USkinnedMeshComponent::SetCastCapsuleDirectShadow(bool bNewValue)
{
	if (bNewValue != bCastCapsuleDirectShadow)
	{
		bCastCapsuleDirectShadow = bNewValue;
		MarkRenderStateDirty();
	}
}

void USkinnedMeshComponent::SetCastCapsuleIndirectShadow(bool bNewValue)
{
	if (bNewValue != bCastCapsuleIndirectShadow)
	{
		bCastCapsuleIndirectShadow = bNewValue;
		MarkRenderStateDirty();
	}
}

void USkinnedMeshComponent::SetCapsuleIndirectShadowMinVisibility(float NewValue)
{
	if (NewValue != CapsuleIndirectShadowMinVisibility)
	{
		CapsuleIndirectShadowMinVisibility = NewValue;
		MarkRenderStateDirty();
	}
}

void USkinnedMeshComponent::SetEvaluateWorldPositionOffset(bool bNewValue)
{
	if (bEvaluateWorldPositionOffset != bNewValue)
	{
		bEvaluateWorldPositionOffset = bNewValue;
		if (GetSkinnedAsset() && GetScene() && SceneProxy)
		{
			SceneProxy->SetEvaluateWorldPositionOffset_GameThread(bNewValue);
			MarkRenderTransformDirty();
		}
	}
}

void USkinnedMeshComponent::SetWorldPositionOffsetDisableDistance(int32 NewValue)
{
	if (WorldPositionOffsetDisableDistance != NewValue)
	{
		WorldPositionOffsetDisableDistance = NewValue;
		if (GetSkinnedAsset() && GetScene() && SceneProxy)
		{
			SceneProxy->SetWorldPositionOffsetDisableDistance_GameThread(NewValue);
			MarkRenderTransformDirty();
		}
	}
}

void USkinnedMeshComponent::SetForceDisableNanite(bool bInForceDisableNanite)
{
	if (bForceDisableNanite == bInForceDisableNanite)
	{
		return;
	}
	bForceDisableNanite = bInForceDisableNanite;

	// Recreate the proxy iff the decision flips.
	if (SceneProxy != nullptr && SceneProxy->IsNaniteMesh() != ShouldNaniteSkin())
	{
		MarkRenderStateDirty();
	}
}

// @todo: think about consolidating this with UpdateLODStatus_Internal
int32 USkinnedMeshComponent::GetDesiredSyncLOD() const
{
	if (GetSkinnedAsset() && MeshObject)
	{
#if WITH_EDITOR
		const int32 LODBias = GetLODBias();
#else
		const int32 LODBias = GSkeletalMeshLODBias;
#endif
		return MeshObject->MinDesiredLODLevel + LODBias;
	}

	return INDEX_NONE;
}

int32 USkinnedMeshComponent::GetBestAvailableLOD() const
{
	if (GetSkinnedAsset() && MeshObject)
	{
		const FSkeletalMeshRenderData& RenderData = MeshObject->GetSkeletalMeshRenderData();
		
		// If PendingFirstLODIdx is higher (worse) than CurrentFirstLODIdx, it means that
		// CurrentFirstLODIdx is about to be streamed out, so we can't rely on it to be available.
		const int32 BestAvailableLOD = FMath::Max<int32>(RenderData.PendingFirstLODIdx, RenderData.CurrentFirstLODIdx);
		check(BestAvailableLOD >= 0);

		return BestAvailableLOD;
	}

	return INDEX_NONE;
}

void USkinnedMeshComponent::SetForceStreamedLOD(int32 LODIndex)
{
	const int32 OldValue = ForceStreamedLodModel;
	// Need to add 1 here, because LODIndex is 0-based but ForceStreamedLodModel is 1-based
	ForceStreamedLodModel = FMath::Clamp(LODIndex + 1, 0, GetNumLODs());
	if (OldValue != ForceStreamedLodModel)
	{
		IStreamingManager::Get().NotifyPrimitiveUpdated(this);
	}
}

void USkinnedMeshComponent::SetForceRenderedLOD(int32 LODIndex)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Need to add 1 here, because LODIndex is 0-based but ForcedLodModel is 1-based
	ForcedLodModel = FMath::Clamp(LODIndex + 1, 0, GetNumLODs());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int32 USkinnedMeshComponent::GetForceRenderedLOD() const
{
	// Subtract 1 because ILODSyncInterface::GetForceRenderedLOD requires the return value to be 
	// 0-based, but the return value of GetForcedLOD() is 1-based for legacy reasons.
	return GetForcedLOD() - 1;
}

int32 USkinnedMeshComponent::GetNumSyncLODs() const
{
	return GetNumLODs();
}

int32 USkinnedMeshComponent::GetForceStreamedLOD() const
{
	// Subtract 1 because ILODSyncInterface::GetForceStreamedLOD requires the return value to be 
	// 0-based, but ForceStreamedLodModel is 1-based for legacy reasons.
	return ForceStreamedLodModel - 1;
}

bool USkinnedMeshComponent::UpdateLODStatus()
{
	return UpdateLODStatus_Internal(INDEX_NONE);
}

bool USkinnedMeshComponent::UpdateLODStatus_Internal(int32 InLeaderPoseComponentPredictedLODLevel, bool bRequestedByLeaderPoseComponent)
{
	// Don't update LOD status for follower component unless it explicitly ignores its leader component's LOD or if update is recursively requested by its leader.
	// This is because when UpdateLODStatus is called on leader component, it updates the follower component LOD,
	// therefore if follower component also calls UpdateLODStatus, it could overturn the result and cause LOD to be out of sync from its leader.
	if (LeaderPoseComponent.IsValid() && !bIgnoreLeaderPoseComponentLOD && !bRequestedByLeaderPoseComponent)
	{
		return false;
	}

	// Predict the best (min) LOD level we are going to need. Basically we use the Min (best) LOD the renderer desired last frame.
	// Because we update bones based on this LOD level, we have to update bones to this LOD before we can allow rendering at it.

	const int32 OldPredictedLODLevel = GetPredictedLODLevel();
	int32 NewPredictedLODLevel = OldPredictedLODLevel;

	if (CVarSkinnedMeshSyncStreamingLODClamp.GetValueOnAnyThread() != 0)
	{
		// Pull parent value which has already been updated if this can use it. Otherwise, recompute using this as root
		if (bSyncAttachParentLOD && GetAttachParent() && GetAttachParent()->IsA<USkinnedMeshComponent>())
		{
			const USkinnedMeshComponent* ParentComponent = Cast<USkinnedMeshComponent>(GetAttachParent());
			MaxFirstLODIdxInAttachmentTree = ParentComponent->MaxFirstLODIdxInAttachmentTree;
		}
		else
		{
			MaxFirstLODIdxInAttachmentTree = ComputeMaxFirstLODIdxInAttachmentTree();
		}
	}
	else
	{
		MaxFirstLODIdxInAttachmentTree = 0;
	}

	if (GetSkinnedAsset() != nullptr)
	{
#if WITH_EDITOR
		const int32 LODBias = GetLODBias();
#else
		const int32 LODBias = GSkeletalMeshLODBias;
#endif

		const int32 MinLodIndex = ComputeMinLOD();
		const int32 MaxLODIndex = FMath::Max(GetNumLODs() - 1, MinLodIndex);

		if (MeshObject)
		{
			MaxDistanceFactor = MeshObject->MaxDistanceFactor;
		}

		// Support forcing to a particular LOD.
		const int32 LocalForcedLodModel = GetForcedLOD();
		if (LocalForcedLodModel > 0)
		{
			NewPredictedLODLevel = FMath::Clamp(LocalForcedLodModel - 1, MinLodIndex, MaxLODIndex);
		}
		else
		{
			// Match LOD of LeaderPoseComponent if it exists.
			if (InLeaderPoseComponentPredictedLODLevel != INDEX_NONE && !bIgnoreLeaderPoseComponentLOD)
			{
				NewPredictedLODLevel = FMath::Clamp(InLeaderPoseComponentPredictedLODLevel, 0, MaxLODIndex);
			}
			else if (bSyncAttachParentLOD && GetAttachParent() && GetAttachParent()->IsA(USkinnedMeshComponent::StaticClass()))
			{
				const USkinnedMeshComponent* ParentComponent = CastChecked<USkinnedMeshComponent>(GetAttachParent());
				NewPredictedLODLevel = FMath::Min(FMath::Max(ParentComponent->GetPredictedLODLevel(), (int32)MaxFirstLODIdxInAttachmentTree), MaxLODIndex);
			}
			else if (MeshObject)
			{
				NewPredictedLODLevel = FMath::Clamp(MeshObject->MinDesiredLODLevel + LODBias, 0, MaxLODIndex);
			}
			// If no MeshObject - just reuse old predicted LOD.
			else
			{
				NewPredictedLODLevel = FMath::Clamp(OldPredictedLODLevel, MinLodIndex, MaxLODIndex);
			}

			// now check to see if we have a MinLODLevel and apply it
			if ((MinLodIndex > 0))
			{
				if(MinLodIndex <= MaxLODIndex)
				{
					NewPredictedLODLevel = FMath::Clamp(NewPredictedLODLevel, MinLodIndex, MaxLODIndex);
				}
				else
				{
					NewPredictedLODLevel = MaxLODIndex;
				}
			}
		}

		if (GetSkinnedAsset()->IsStreamable() && MeshObject)
		{
			// LOD streaming is enabled for this mesh, so clamp the predicted LOD to the best LOD
			// that is loaded and is not waiting to be streamed out.

			const FSkeletalMeshRenderData& RenderData = MeshObject->GetSkeletalMeshRenderData();
			NewPredictedLODLevel = FMath::Max<int32>(NewPredictedLODLevel, RenderData.PendingFirstLODIdx);
			NewPredictedLODLevel = FMath::Max<int32>(NewPredictedLODLevel, RenderData.CurrentFirstLODIdx);
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (CVarAnimVisualizeLODs.GetValueOnAnyThread() != 0)
		{
			// Reduce to visible animated, non SyncAttachParentLOD to reduce clutter.
			if (GetSkinnedAsset() && MeshObject && bRecentlyRendered)
			{
				const bool bHasValidSyncAttachParent = bSyncAttachParentLOD && GetAttachParent() && GetAttachParent()->IsA(USkinnedMeshComponent::StaticClass());
				if (!bHasValidSyncAttachParent)
				{
					const float ScreenSize = FMath::Sqrt(MeshObject->MaxDistanceFactor) * 2.f;
					FString DebugString = FString::Printf(TEXT("PredictedLODLevel(%d)\nMinDesiredLODLevel(%d) ForcedLodModel(%d) MinLodIndex(%d) LODBias(%d)\nMaxDistanceFactor(%f) ScreenSize(%f)"),
						GetPredictedLODLevel(), MeshObject->MinDesiredLODLevel, LocalForcedLodModel, MinLodIndex, LODBias, MeshObject->MaxDistanceFactor, ScreenSize);

					// See if Child classes want to add something.
					UpdateVisualizeLODString(DebugString);

					FColor DrawColor = FColor::White;
					switch (GetPredictedLODLevel())
					{
					case 0: DrawColor = FColor::White; break;
					case 1: DrawColor = FColor::Green; break;
					case 2: DrawColor = FColor::Yellow; break;
					case 3: DrawColor = FColor::Red; break;
					default:
						DrawColor = FColor::Purple; break;
					}

					DrawDebugString(GetWorld(), Bounds.Origin, DebugString, nullptr, DrawColor, 0.f, true, 1.2f);
				}
			}
		}
#endif
	}
	else
	{
		NewPredictedLODLevel = 0;
	}

	// See if LOD has changed. 
	bool bLODChanged = (NewPredictedLODLevel != OldPredictedLODLevel);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PredictedLODLevel = NewPredictedLODLevel;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// also update follower component LOD status, as we may need to recalc required bones if this changes
	// independently of our LOD
	for (const TWeakObjectPtr<USkinnedMeshComponent>& FollowerComponent : FollowerPoseComponents)
	{
		if (USkinnedMeshComponent* FollowerComponentPtr = FollowerComponent.Get())
		{
			bLODChanged |= FollowerComponentPtr->UpdateLODStatus_Internal(NewPredictedLODLevel, /*bRequestedByLeaderPoseComponent=*/true);
		}
	}

	return bLODChanged;
}

uint8 USkinnedMeshComponent::ComputeMaxFirstLODIdxInAttachmentTree() const
{
	const TArray<TObjectPtr<USceneComponent>>& LocalAttachChildren = GetAttachChildren();
	uint8 Result = 0;
	
	for (const TObjectPtr<USceneComponent>& AttachChild : LocalAttachChildren)
	{
		const USkinnedMeshComponent* AttachSkinnedMeshComponent = Cast<USkinnedMeshComponent>(AttachChild);
		if (AttachSkinnedMeshComponent && AttachSkinnedMeshComponent->bSyncAttachParentLOD)
		{
			const uint8 ChildResult = AttachSkinnedMeshComponent->ComputeMaxFirstLODIdxInAttachmentTree();
			Result = FMath::Max(Result, ChildResult);
		}
	}

	if (GetSkinnedAsset() && GetSkinnedAsset()->IsStreamable() && MeshObject)
	{
		const FSkeletalMeshRenderData& RenderData = MeshObject->GetSkeletalMeshRenderData();
		Result = FMath::Max(Result, RenderData.PendingFirstLODIdx);
		Result = FMath::Max(Result, RenderData.CurrentFirstLODIdx);
	}

	return Result;
}

void USkinnedMeshComponent::FinalizeBoneTransform()
{
	FlipEditableSpaceBases();
	// we finalized bone transform, now we have valid bone buffer
	bHasValidBoneTransform = true;

#if WITH_STATE_STREAM_ACTOR
	if (TransformHandle.IsValid())
	{
		FTransformDynamicState Ds;
		Ds.SetBoneTransforms(GetComponentSpaceTransforms());
		const_cast<FTransformHandle&>(TransformHandle).Update(Ds);
	}
#endif

}

void USkinnedMeshComponent::FlipEditableSpaceBases()
{
	if (bNeedToFlipSpaceBaseBuffers)
	{
		bNeedToFlipSpaceBaseBuffers = false;

		if (bDoubleBufferedComponentSpaceTransforms)
		{
			CurrentReadComponentTransforms = CurrentEditableComponentTransforms;
			CurrentEditableComponentTransforms = 1 - CurrentEditableComponentTransforms;

			// copy to other buffer if we dont already have a valid set of transforms
			if (!bHasValidBoneTransform)
			{
				GetEditableComponentSpaceTransforms() = GetComponentSpaceTransforms();
				GetEditableBoneVisibilityStates() = GetBoneVisibilityStates();
				bBoneVisibilityDirty = false;
			}
			// If we have changed bone visibility, then we need to reflect that next frame
			else if(bBoneVisibilityDirty)
			{
				GetEditableBoneVisibilityStates() = GetBoneVisibilityStates();
				bBoneVisibilityDirty = false;
			}
		}
		else
		{
			// save previous transform if it's valid
			if (bHasValidBoneTransform)
			{
				PreviousComponentSpaceTransformsArray = GetComponentSpaceTransforms();
				PreviousBoneVisibilityStates = GetBoneVisibilityStates();
			}

			CurrentReadComponentTransforms = CurrentEditableComponentTransforms = 0;

			// if we don't have a valid transform, we copy after we write, so that it doesn't cause motion blur
			if (!bHasValidBoneTransform)
			{
				PreviousComponentSpaceTransformsArray = GetComponentSpaceTransforms();
				PreviousBoneVisibilityStates = GetBoneVisibilityStates();
			}
		}

		BoneTransformUpdateMethodQueue.Add(EBoneTransformUpdateMethod::AnimationUpdate);
		// Bone revision number needs to be updated immediately, because dynamic updates on components are run in parallel later,
		// for a follower component it relies on its lead component to be up-to-date, so updating the lead component revision number here guarantees it.
		UpdateBoneTransformRevisionNumber();
	}
}

void USkinnedMeshComponent::SetComponentSpaceTransformsDoubleBuffering(bool bInDoubleBufferedComponentSpaceTransforms)
{
	bDoubleBufferedComponentSpaceTransforms = bInDoubleBufferedComponentSpaceTransforms;

	if (bDoubleBufferedComponentSpaceTransforms)
	{
		CurrentEditableComponentTransforms = 1 - CurrentReadComponentTransforms;
	}
	else
	{
		CurrentEditableComponentTransforms = CurrentReadComponentTransforms = 0;
	}
}

void USkinnedMeshComponent::GetCPUSkinnedVertices(TArray<FFinalSkinVertex>& OutVertices, int32 InLODIndex) const
{
	// Work around the fact that non-const methods must be called to perform the skinning
	// Component state should be left unchanged at the end of this call.
	USkinnedMeshComponent* MutableThis = const_cast<USkinnedMeshComponent*>(this);

	USkinnedMeshComponent* PoseComponent = MutableThis->LeaderPoseComponent.Get() ? MutableThis->LeaderPoseComponent.Get() : MutableThis;

	int32 CachedForcedLOD = PoseComponent->GetForcedLOD();
	PoseComponent->SetForcedLOD(InLODIndex + 1);
	PoseComponent->UpdateLODStatus();
	PoseComponent->RefreshBoneTransforms(nullptr);
	
	// Turn bRenderStatic off so MeshObject can be switched to FSkeletalMeshObjectCPUSkin
	const bool bCachedRenderStatic = bRenderStatic;
	MutableThis->bRenderStatic = false;

	// switch to CPU skinning
	const bool bCachedCPUSkinning = GetCPUSkinningEnabled();
	constexpr bool bRecreateRenderStateImmediately = true;
	MutableThis->SetCPUSkinningEnabled(true, bRecreateRenderStateImmediately);

	check(MeshObject);
	check(MeshObject->IsCPUSkinned());
		
	GetCPUSkinnedCachedFinalVertices(OutVertices);
	
	// switch skinning mode, LOD, bRenderStatic, etc. back
	PoseComponent->SetForcedLOD(CachedForcedLOD);
	MutableThis->bRenderStatic = bCachedRenderStatic;
	MutableThis->SetCPUSkinningEnabled(bCachedCPUSkinning, bRecreateRenderStateImmediately);
}

void USkinnedMeshComponent::GetCPUSkinnedCachedFinalVertices(TArray<FFinalSkinVertex>& OutVertices) const
{
	if (MeshObject && MeshObject->IsCPUSkinned())
	{
		// Copy our vertices out. We know we are using CPU skinning now, so this cast is safe
		OutVertices = static_cast<FSkeletalMeshObjectCPUSkin*>(MeshObject)->GetCachedFinalVertices();
	}
}

const Nanite::FResources* USkinnedMeshComponent::GetNaniteResources() const
{
	return FSkinnedMeshComponentHelper::GetNaniteResources(*this);
}

bool USkinnedMeshComponent::HasValidNaniteData() const
{
	return FSkinnedMeshComponentHelper::HasValidNaniteData(*this);
}

void USkinnedMeshComponent::ReleaseResources()
{
	for (int32 LODIndex = 0; LODIndex < LODInfo.Num(); LODIndex++)
	{
		LODInfo[LODIndex].BeginReleaseOverrideVertexColors();
		LODInfo[LODIndex].BeginReleaseOverrideSkinWeights();
	}

	DetachFence.BeginFence();
}

void USkinnedMeshComponent::RegisterLODStreamingCallback(FLODStreamingCallback&& Callback, int32 LODIdx, float TimeoutSecs, bool bOnStreamIn)
{
	if (GetSkinnedAsset())
	{
		if (LODIdx < 0)
		{
			LODIdx = GetSkinnedAsset()->GetMinLodIdx(true);
		}
		GetSkinnedAsset()->RegisterMipLevelChangeCallback(GetPrimitiveComponentInterface(), LODIdx, TimeoutSecs, bOnStreamIn, MoveTemp(Callback));
		bMipLevelCallbackRegistered = true;
	}
}

void USkinnedMeshComponent::RegisterLODStreamingCallback(FLODStreamingCallback&& CallbackStreamingStart, FLODStreamingCallback&& CallbackStreamingDone, float TimeoutStartSecs, float TimeoutDoneSecs)
{
	if (GetSkinnedAsset())
	{
		GetSkinnedAsset()->RegisterMipLevelChangeCallback(GetPrimitiveComponentInterface(), TimeoutStartSecs, MoveTemp(CallbackStreamingStart), TimeoutDoneSecs, MoveTemp(CallbackStreamingDone));
		bMipLevelCallbackRegistered = true;
	}
}

bool USkinnedMeshComponent::PrestreamMeshLODs(float Seconds)
{
	if (USkinnedAsset* Asset = GetSkinnedAsset())
	{
		static IConsoleVariable* CVarAllowFastForceResident = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Streaming.AllowFastForceResident"));
		Asset->bIgnoreStreamingMipBias = CVarAllowFastForceResident && CVarAllowFastForceResident->GetInt();
		Asset->SetForceMipLevelsToBeResident(Seconds);
		return IStreamingManager::Get().GetRenderAssetStreamingManager().FastForceFullyResident(Asset);
	}
	return false;
}

void USkinnedMeshComponent::BeginDestroy()
{
	if (GetSkinnedAsset() && bMipLevelCallbackRegistered)
	{
		GetSkinnedAsset()->RemoveMipLevelChangeCallback(this);
		bMipLevelCallbackRegistered = false;
	}

	Super::BeginDestroy();
	ReleaseResources();

#if WITH_EDITOR
	// The object cache needs to be notified when we're getting destroyed
	FObjectCacheEventSink::NotifySkinnedMeshChanged_Concurrent(GetSkinnedMeshComponentInterface());
#endif // WITH_EDITOR

	if (bSkinWeightProfilePending)
	{
		bSkinWeightProfilePending = false;
		if (FSkinWeightProfileManager * Manager = FSkinWeightProfileManager::Get(GetWorld()))
		{
			Manager->CancelSkinWeightProfileRequest(this);
		}
	}

	// Release ref pose override if allocated
	if (RefPoseOverride)
	{
		RefPoseOverride = nullptr;
	}

	// Disconnect follower components from this component if present.
	// They will currently have no transforms allocated so will be
	// in an invalid state when this component is destroyed
	// Walk backwards as we'll be removing from this array
	const int32 NumFollowerComponents = FollowerPoseComponents.Num();
	for(int32 FollowerIndex = NumFollowerComponents - 1; FollowerIndex >= 0; --FollowerIndex)
	{
		if(USkinnedMeshComponent* Follower = FollowerPoseComponents[FollowerIndex].Get())
		{
			Follower->SetLeaderPoseComponent(nullptr);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FSkelMeshComponentLODInfo::FSkelMeshComponentLODInfo()
{
}

FSkelMeshComponentLODInfo::~FSkelMeshComponentLODInfo()
{
	CleanUpOverrideVertexColors();
	CleanUpOverrideSkinWeights();
}

void FSkelMeshComponentLODInfo::SetActiveSkinWeightProfileStack(const FSkinWeightProfileStack& InProfileStack)
{
	ProfileStackRequest = InProfileStack;
}

bool FSkelMeshComponentLODInfo::GetActiveSkinWeightProfileStack(FSkinWeightProfileStack& OutProfileStack) const
{
	OutProfileStack = ProfileStackRequest;
	return !ProfileStackRequest.IsEmpty();
}


void FSkelMeshComponentLODInfo::ReleaseOverrideVertexColorsAndBlock()
{
	if (OverrideVertexColors)
	{
		// enqueue a rendering command to release
		BeginReleaseResource(OverrideVertexColors, &UE::RenderCommandPipe::SkeletalMesh);
		// Ensure the RT no longer accessed the data, might slow down
		FlushRenderingCommands();
		// The RT thread has no access to it any more so it's safe to delete it.
		CleanUpOverrideVertexColors();
	}
}

bool FSkelMeshComponentLODInfo::BeginReleaseOverrideVertexColors()
{
	if (OverrideVertexColors)
	{
		// enqueue a rendering command to release
		BeginReleaseResource(OverrideVertexColors, &UE::RenderCommandPipe::SkeletalMesh);
		return true;
	}

	return false;
}

void FSkelMeshComponentLODInfo::EndReleaseOverrideVertexColors()
{
	CleanUpOverrideVertexColors();
}

void FSkelMeshComponentLODInfo::CleanUpOverrideVertexColors()
{
	if (OverrideVertexColors)
	{
		delete OverrideVertexColors;
		OverrideVertexColors = nullptr;
	}
}

void FSkelMeshComponentLODInfo::ReleaseOverrideSkinWeightsAndBlock()
{
	if (OverrideSkinWeights)
	{
		// enqueue a rendering command to release
		OverrideSkinWeights->BeginReleaseResources();
		// Ensure the RT no longer accessed the data, might slow down
		FlushRenderingCommands();
		// The RT thread has no access to it any more so it's safe to delete it.
		CleanUpOverrideSkinWeights();
	}
}

bool FSkelMeshComponentLODInfo::BeginReleaseOverrideSkinWeights()
{
	if (OverrideSkinWeights)
	{
		// enqueue a rendering command to release
		OverrideSkinWeights->BeginReleaseResources();
		return true;
	}

	return false;
}

void FSkelMeshComponentLODInfo::EndReleaseOverrideSkinWeights()
{
	CleanUpOverrideSkinWeights();
}

void FSkelMeshComponentLODInfo::CleanUpOverrideSkinWeights()
{
	if (OverrideSkinWeights)
	{
		delete OverrideSkinWeights;
		OverrideSkinWeights = nullptr;
	}

	if (OverrideProfileSkinWeights)
	{
		OverrideProfileSkinWeights = nullptr;
	}
}

//////////////////////////////////////////////////////////////////////////

void USkinnedMeshComponent::SetVertexColorOverride_LinearColor(int32 LODIndex, const TArray<FLinearColor>& VertexColors)
{
	TArray<FColor> Colors;
	if (VertexColors.Num() > 0)
	{
		Colors.SetNum(VertexColors.Num());

		for (int32 ColorIdx = 0; ColorIdx < VertexColors.Num(); ColorIdx++)
		{
			Colors[ColorIdx] = VertexColors[ColorIdx].ToFColor(false);
		}
	}
	SetVertexColorOverride(LODIndex, Colors);
}


void USkinnedMeshComponent::SetVertexColorOverride(int32 LODIndex, const TArray<FColor>& VertexColors)
{
	LLM_SCOPE_BYNAME(TEXT("SkeletalMesh/VertexColorOverride"));

	InitLODInfos();

	FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();

	// If we have a render resource, and the requested LODIndex is valid (for both component and mesh, though these should be the same)
	if (SkelMeshRenderData != nullptr && LODInfo.IsValidIndex(LODIndex) && SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex))
	{
		ensure(LODInfo.Num() == SkelMeshRenderData->LODRenderData.Num());

		FSkelMeshComponentLODInfo& Info = LODInfo[LODIndex];
		if (Info.OverrideVertexColors != nullptr)
		{
			Info.ReleaseOverrideVertexColorsAndBlock();
		}

		const TArray<FColor>* UseColors;
		TArray<FColor> ResizedColors;

		FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];
		const int32 ExpectedNumVerts = LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();

		// If colors passed in are correct size, just use them
		if (VertexColors.Num() == ExpectedNumVerts)
		{
			UseColors = &VertexColors;
		}
		// If not the correct size, resize to correct size
		else
		{
			// presize array
			ResizedColors.AddUninitialized(ExpectedNumVerts);

			// Copy while input and output are valid
			int32 VertCount = 0;
			while (VertCount < ExpectedNumVerts)
			{
				if (VertCount < VertexColors.Num())
				{
					ResizedColors[VertCount] = VertexColors[VertCount];
				}
				else
				{
					ResizedColors[VertCount] = FColor::White;
				}

				VertCount++;
			}

			UseColors = &ResizedColors;
		}

		Info.OverrideVertexColors = new FColorVertexBuffer;
		Info.OverrideVertexColors->InitFromColorArray(*UseColors);

		BeginInitResource(Info.OverrideVertexColors, &UE::RenderCommandPipe::SkeletalMesh);

		MarkRenderStateDirty();
	}
}

void USkinnedMeshComponent::ClearVertexColorOverride(int32 LODIndex)
{
	// If we have a render resource, and the requested LODIndex is valid (for both component and mesh, though these should be the same)
	if (LODInfo.IsValidIndex(LODIndex))
	{
		FSkelMeshComponentLODInfo& Info = LODInfo[LODIndex];
		if (Info.OverrideVertexColors != nullptr)
		{
			Info.ReleaseOverrideVertexColorsAndBlock();
			MarkRenderStateDirty();
		}
	}
}

/** 
 *	Util for converting from API skin weight description to GPU format. 
 * 	This includes remapping from skeleton bone index to section bone index.
 */
void CreateSectionSkinWeightsArray(
	const TArray<FSkelMeshSkinWeightInfo>& InSourceWeights,
	int32 StartIndex,
	int32 NumVerts,
	const TMap<int32, int32>& SkelToSectionBoneMap,
	TArray<FSkinWeightInfo>& OutGPUWeights,
	TArray<int32>& OutInvalidBones)
{
	OutGPUWeights.AddUninitialized(NumVerts);

	TArray<int32> InvalidBones;

	bool bWeightUnderrun = false;
	// Iterate over new output buffer
	for(int VertIndex = StartIndex; VertIndex < StartIndex + NumVerts; VertIndex++)
	{
		FSkinWeightInfo& TargetWeight = OutGPUWeights[VertIndex];
		// while we have valid entries in input buffer
		if (VertIndex < InSourceWeights.Num())
		{
			const FSkelMeshSkinWeightInfo& SrcWeight = InSourceWeights[VertIndex];

			// Iterate over influences
			for (int32 InfIndex = 0; InfIndex < MAX_TOTAL_INFLUENCES; InfIndex++)
			{
				// init to zero
				TargetWeight.InfluenceBones[InfIndex] = 0;
				TargetWeight.InfluenceWeights[InfIndex] = 0;

				// if we have a valid weight, see if we have a valid bone mapping for desired bone
				// Map from 8-bit weights to 16-bit by aligning the min/max range.
				const uint16 InfWeight = (SrcWeight.Weights[InfIndex] << 8) | SrcWeight.Weights[InfIndex];
				if (InfWeight > 0)
				{
					const int32 SkelBoneIndex = SrcWeight.Bones[InfIndex];
					const int32* SectionBoneIndexPtr = SkelToSectionBoneMap.Find(SkelBoneIndex);

					// We do, use remapped value and copy weight
					if (SectionBoneIndexPtr)
					{
						TargetWeight.InfluenceBones[InfIndex] = *SectionBoneIndexPtr;
						TargetWeight.InfluenceWeights[InfIndex] = InfWeight;
					}
					// We don't, we'll warn, and leave zeros (this will mess up mesh, but not clear how to resolve this...)
					else
					{
						OutInvalidBones.AddUnique(SkelBoneIndex);
					}
				}
			}
		}
		// Oops, 
		else
		{
			bWeightUnderrun = true;

			TargetWeight.InfluenceBones[0] = 0;
			TargetWeight.InfluenceWeights[0] = UE::AnimationCore::MaxRawBoneWeight;

			for (int32 InfIndex = 1; InfIndex < MAX_TOTAL_INFLUENCES; InfIndex++)
			{
				TargetWeight.InfluenceBones[InfIndex] = 0;
				TargetWeight.InfluenceWeights[InfIndex] = 0;
			}
		}
	}

	if (bWeightUnderrun)
	{
		UE_LOGF(LogSkinnedMeshComp, Warning, "SetSkinWeightOverride: Too few weights specified.");
	}
}

void CreateSkinWeightsArray(
	const TArray<FSkelMeshSkinWeightInfo>& InSourceWeights,
	FSkeletalMeshLODRenderData& LODData,
	TArray<FSkinWeightInfo>& OutGPUWeights,
	const FReferenceSkeleton& RefSkel)
{
	// Index of first vertex in current section, in the big overall buffer
	int32 BaseVertIndex = 0;
	for (int32 SectionIdx = 0; SectionIdx < LODData.RenderSections.Num(); SectionIdx++)
	{
		const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIdx];
		const int32 NumVertsInSection = Section.NumVertices;

		// Build inverse mapping from skeleton bone index to section vertex index
		TMap<int32, int32> SkelToSectionBoneMap;
		for (int32 i = 0; i < Section.BoneMap.Num(); i++)
		{
			SkelToSectionBoneMap.Add(Section.BoneMap[i], i);
		}

		// Convert skin weight struct format and assign to new vertex buffer
		TArray<int32> InvalidBones;
		CreateSectionSkinWeightsArray(InSourceWeights, BaseVertIndex, NumVertsInSection, SkelToSectionBoneMap, OutGPUWeights, InvalidBones);

		// Log info for invalid bones
		if (InvalidBones.Num() > 0)
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "SetSkinWeightOverride: Invalid bones index specified for section %d:", SectionIdx);

			for (int32 BoneIndex : InvalidBones)
			{
				FName BoneName = RefSkel.GetBoneName(BoneIndex);
				UE_LOGF(LogSkinnedMeshComp, Warning, "SetSkinWeightOverride: %d %ls", BoneIndex, *BoneName.ToString());
			}
		}

		BaseVertIndex += NumVertsInSection;
	}
}


void USkinnedMeshComponent::SetSkinWeightOverride(int32 LODIndex, const TArray<FSkelMeshSkinWeightInfo>& SkinWeights)
{
	LLM_SCOPE_BYNAME(TEXT("SkeletalMesh/SkinWeightOverride"));

	InitLODInfos();

	FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();

	// If we have a render resource, and the requested LODIndex is valid (for both component and mesh, though these should be the same)
	if (SkelMeshRenderData != nullptr && LODInfo.IsValidIndex(LODIndex) && SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex))
	{
		ensure(LODInfo.Num() == SkelMeshRenderData->LODRenderData.Num());

		FSkelMeshComponentLODInfo& Info = LODInfo[LODIndex];
		if (Info.OverrideSkinWeights != nullptr)
		{
			Info.ReleaseOverrideSkinWeightsAndBlock();
		}

		FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];
		const int32 ExpectedNumVerts = LODData.GetNumVertices();

		// Only proceed if we have enough weights (we can proceed if we have too many)
		if (SkinWeights.Num() >= ExpectedNumVerts)
		{
			if (SkinWeights.Num() > ExpectedNumVerts)
			{
				UE_LOGF(LogSkinnedMeshComp, Warning, "SetSkinWeightOverride: Too many weights - expected %d, got %d - truncating", ExpectedNumVerts, SkinWeights.Num());
			}

			uint32 NumBoneInfluences = LODData.GetVertexBufferMaxBoneInfluences();
			bool bUse16BitBoneIndex = LODData.DoesVertexBufferUse16BitBoneIndex();

			// Allocate skin weight override buffer
			Info.OverrideSkinWeights = new FSkinWeightVertexBuffer;
			Info.OverrideSkinWeights->SetNeedsCPUAccess(true);
			Info.OverrideSkinWeights->SetMaxBoneInfluences(NumBoneInfluences);
			Info.OverrideSkinWeights->SetUse16BitBoneIndex(bUse16BitBoneIndex);

			const USkinnedAsset* Mesh = GetSkinnedAsset();
#if RHI_ENABLE_RESOURCE_INFO
			const FName OwnerName(USkinnedAsset::GetLODPathName(Mesh, LODIndex));
			Info.OverrideSkinWeights->SetOwnerName(OwnerName);
#endif

			const FReferenceSkeleton& RefSkel = Mesh->GetRefSkeleton();
			TArray<FSkinWeightInfo> GPUWeights;
			CreateSkinWeightsArray(SkinWeights, LODData, GPUWeights, RefSkel);
			*(Info.OverrideSkinWeights) = GPUWeights;
			Info.OverrideSkinWeights->BeginInitResources();

			MarkRenderStateDirty();
		}
		else
		{
			UE_LOGF(LogSkinnedMeshComp, Warning, "SetSkinWeightOverride: Not enough weights - expected %d, got %d - aborting.", ExpectedNumVerts, SkinWeights.Num());
		}
	}
}

void USkinnedMeshComponent::ClearSkinWeightOverride(int32 LODIndex)
{
	SCOPED_NAMED_EVENT(USkinnedMeshComponent_ClearSkinWeightOverride, FColor::Yellow);

	// If we have a render resource, and the requested LODIndex is valid (for both component and mesh, though these should be the same)
	if (LODInfo.IsValidIndex(LODIndex))
	{
		FSkelMeshComponentLODInfo& Info = LODInfo[LODIndex];
		if (Info.OverrideSkinWeights != nullptr)
		{
			Info.ReleaseOverrideSkinWeightsAndBlock();
			MarkRenderStateDirty();
		}
	}
}


bool USkinnedMeshComponent::SetSkinWeightProfile(
	FName InProfileName,
	ESkinWeightProfileLayer InLayer
	)
{
	FSkinWeightProfileStack ProfileStack{CurrentSkinWeightProfileLayers};
	ProfileStack[static_cast<int32>(InLayer)] = InProfileName;
	return SetSkinWeightProfileStack(ProfileStack);
}

void USkinnedMeshComponent::ClearSkinWeightProfile(
	ESkinWeightProfileLayer InLayer
	)
{
	FSkinWeightProfileStack ProfileStack{CurrentSkinWeightProfileLayers};
	ProfileStack[static_cast<int32>(InLayer)] = NAME_None;
	SetSkinWeightProfileStack(ProfileStack);
}


void USkinnedMeshComponent::ClearAllSkinWeightProfiles()
{
	SetSkinWeightProfileStack(FSkinWeightProfileStack());
}


void USkinnedMeshComponent::UnloadSkinWeightProfile(FName InProfileName)
{
	#if 0
	if (FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData())
	{
		if (LODInfo.Num())
		{
			bool bCleared = false;
			for (int32 LODIndex = 0; LODIndex < LODInfo.Num(); ++LODIndex)
			{
				// Queue release and deletion of the skin weight buffer associated with the profile name
				FSkeletalMeshLODRenderData& RenderData = SkelMeshRenderData->LODRenderData[LODIndex];
				RenderData.SkinWeightProfilesData.ReleaseBuffer(InProfileName);

				// In case the buffer previously released is currently set for this component, clear it
				if (CurrentSkinWeightProfileName == InProfileName)
				{
					FSkelMeshComponentLODInfo& Info = LODInfo[LODIndex];
					Info.OverrideProfileSkinWeights = nullptr;
					bCleared = true;
				}
			}

			if (bCleared)
			{
				UpdateSkinWeightOverrideBuffer();
			}
		}

		if (bSkinWeightProfilePending)
		{
			if (FSkinWeightProfileManager * Manager = FSkinWeightProfileManager::Get(GetWorld()))
			{
				Manager->CancelSkinWeightProfileRequest(this);
			}

			bSkinWeightProfilePending = false;
		}
	}

	if (CurrentSkinWeightProfileName == InProfileName)
	{
		bSkinWeightProfileSet = false;
		CurrentSkinWeightProfileName = NAME_None;
	}
#endif
}

bool USkinnedMeshComponent::IsUsingSkinWeightProfile() const
{
	if (bSkinWeightProfileSet)
	{		
		return true;
	}

	if (USkinnedAsset* Asset = GetSkinnedAsset())
	{
		if (Asset->IsCompiling())
		{
			return false;
		}

		if (const FSkinWeightProfilesData* ProfileData = Asset->GetSkinWeightProfilesData(GetPredictedLODLevel()))
		{
			return ProfileData->IsDefaultOverridden() || ProfileData->IsStaticOverridden();
		}
	}	

	return false;
}

FName USkinnedMeshComponent::GetCurrentSkinWeightProfileName(
	ESkinWeightProfileLayer InLayer
	) const
{
	const int32 LayerIndex = static_cast<int32>(InLayer);
	return GetCurrentSkinWeightProfileLayerNames()[LayerIndex];
}

TArray<FName> USkinnedMeshComponent::GetCurrentSkinWeightProfileLayerNames() const
{
	FSkinWeightProfileStack ProfileStack;
	if (bSkinWeightProfileSet)
	{
		ProfileStack = FSkinWeightProfileStack{CurrentSkinWeightProfileLayers};
	}

	TArray<FName> ProfileNames;
	ProfileNames.SetNum(FSkinWeightProfileStack::MaxLayerCount);
	
	if (USkinnedAsset* Asset = GetSkinnedAsset())
	{
		if (Asset->IsCompiling())
		{
			//Return an array with name none entries for all layers
			return ProfileNames;
		}

		if (const FSkinWeightProfilesData* ProfileData = Asset->GetSkinWeightProfilesData(GetPredictedLODLevel()))
		{
			if( ProfileData->IsDefaultOverridden() || ProfileData->IsStaticOverridden())
			{
				ProfileStack = ProfileData->GetDefaultProfileStack();
			}
		}
	}

	for (int32 LayerIndex = 0; LayerIndex < FSkinWeightProfileStack::MaxLayerCount; ++LayerIndex)
	{
		ProfileNames[LayerIndex] = ProfileStack[LayerIndex];
	}
	return ProfileNames;
}

bool USkinnedMeshComponent::IsSkinWeightProfileManagementOnRenderThread() const
{
	return bSkinWeightProfileOnRenderThread || IsSkinnedMeshSkinWeightProfilesOnRenderThread();
}

void USkinnedMeshComponent::UpdateSkinWeightOverrideBuffer()
{
	// Force a mesh update to ensure bone buffers are up to date
	bForceMeshObjectUpdate = true;
	MarkRenderStateDirty();

	// Queue an update of the skin weight buffer used by the current Mesh Object
	if (MeshObject)
	{
		MeshObject->UpdateSkinWeightBuffer(this);
	}
}

bool USkinnedMeshComponent::SetSkinWeightProfileStack(const FSkinWeightProfileStack& InProfileStack)
{
	// Normalize the profile stack to remove duplicates.
	FSkinWeightProfileStack NormalizedProfileStack(InProfileStack.Normalized());
	
	// If we're not actually changing anything, then just return and say we did, as if the operation succeeded.
	if (NormalizedProfileStack == FSkinWeightProfileStack{CurrentSkinWeightProfileLayers}.Normalized())
	{
		return true;
	}
	
	bool bChanged = false;
	
	if (FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData())
	{
		if (IsSkinWeightProfileManagementOnRenderThread())
		{
			InitLODInfos();
			// Clear skin weight buffer set for all of the LODs
			const int32 LODStartIndex = FMath::Max(0, GSkinWeightProfilesAllowedFromLOD + 1);
			for (int32 LODIndex = LODStartIndex; LODIndex < LODInfo.Num(); ++LODIndex)
			{
				FSkelMeshComponentLODInfo& Info = LODInfo[LODIndex];
				FSkeletalMeshLODRenderData& LODRenderData = SkelMeshRenderData->LODRenderData[LODIndex];
						
				FSkinWeightProfileStack ActiveProfileStack;
				Info.GetActiveSkinWeightProfileStack(ActiveProfileStack);
				if (ActiveProfileStack != NormalizedProfileStack)
				{
					Info.SetActiveSkinWeightProfileStack(NormalizedProfileStack);
					bChanged = true;
				}
			}
			
			if (NormalizedProfileStack.IsEmpty() && bChanged)
			{
				// If skin weight profile is being "reset" i.e set to empty,
				// update the override buffer immediately. 
				UpdateSkinWeightOverrideBuffer();
			}
			else if (bChanged)
			{
				bSkinWeightProfilePending = true; 
			}
		}
		else
		{
			if (NormalizedProfileStack.IsEmpty())
			{
				if (bSkinWeightProfileSet)
				{
					bool bCleared = false;
					
					InitLODInfos();
					// Clear skin weight buffer set for all of the LODs
					for (int32 LODIndex = 0; LODIndex < LODInfo.Num(); ++LODIndex)
					{
						FSkelMeshComponentLODInfo& Info = LODInfo[LODIndex];
						bCleared |= (Info.OverrideProfileSkinWeights != nullptr);
						Info.OverrideProfileSkinWeights = nullptr;
					}

					if (bCleared)
					{
						UpdateSkinWeightOverrideBuffer();
					}
				}

				// If there was a change request in flight, cancel it now so that we don't get updated with a stale request.
				if (bSkinWeightProfilePending)
				{
					if (FSkinWeightProfileManager* Manager = FSkinWeightProfileManager::Get(GetWorld()))
					{
						Manager->CancelSkinWeightProfileRequest(this);
					}
				}

				bChanged = true;
			}
			else
			{
				// Ensure the LOD infos array is initialized
				bool bHaveExistingBuffer = false;
				
				InitLODInfos();
				for (int32 LODIndex = 0; LODIndex < LODInfo.Num(); ++LODIndex)
				{
					// Check whether setting a profile is allowed for this LOD index
					if (LODIndex > GSkinWeightProfilesAllowedFromLOD)
					{
						FSkeletalMeshLODRenderData& RenderData = SkelMeshRenderData->LODRenderData[LODIndex];

						{
							PRAGMA_DISABLE_DEPRECATION_WARNINGS
							UE::TScopeLock Lock(RenderData.RenderDataMutex);
							PRAGMA_ENABLE_DEPRECATION_WARNINGS

								for (int32 LayerIndex = 0; LayerIndex < FSkinWeightProfileStack::MaxLayerCount; ++LayerIndex)
								{
									bChanged |= RenderData.SkinWeightProfilesData.ContainsProfile(NormalizedProfileStack[LayerIndex]);
								}

							// Retrieve this profile's skin weight buffer
							FSkinWeightVertexBuffer* Buffer = RenderData.SkinWeightProfilesData.GetOverrideBuffer(NormalizedProfileStack);

							FSkelMeshComponentLODInfo& Info = LODInfo[LODIndex];
							Info.OverrideProfileSkinWeights = Buffer;

							bHaveExistingBuffer = (Buffer != nullptr);
						}
					}
				}

				if (bChanged)
				{
					if (bHaveExistingBuffer)
					{
						UpdateSkinWeightOverrideBuffer();
						bSkinWeightProfileSet = true;
					}
					else 
					{
						TWeakObjectPtr<USkinnedMeshComponent> WeakComponent = this;
						
						FRequestFinished Callback = [WeakComponent](TWeakObjectPtr<USkeletalMesh> WeakMesh, FSkinWeightProfileStack ProfileStack)
							{
								// Ensure that the request objects are still valid
								if (WeakMesh.IsValid() && WeakComponent.IsValid())
								{
									USkinnedMeshComponent* Component = WeakComponent.Get();
									Component->InitLODInfos();

									Component->bSkinWeightProfilePending = false;
									Component->bSkinWeightProfileSet = true;

									if (FSkeletalMeshRenderData * RenderData = WeakMesh->GetResourceForRendering())
									{
										const int32 NumLODs = RenderData->LODRenderData.Num();
										for (int32 Index = 0; Index < NumLODs; ++Index)
										{
											FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[Index];

											PRAGMA_DISABLE_DEPRECATION_WARNINGS
											UE::TScopeLock Lock(LODRenderData.RenderDataMutex);
											PRAGMA_ENABLE_DEPRECATION_WARNINGS

											FSkinWeightProfilesData& SkinweightData = LODRenderData.SkinWeightProfilesData;

											// Check whether or not setting a profile is allow for this LOD index
											if (Index > GSkinWeightProfilesAllowedFromLOD)
											{
												// Retrieve this profile's skin weight buffer
												FSkinWeightVertexBuffer* Buffer = SkinweightData.GetOverrideBuffer(ProfileStack);
												FSkelMeshComponentLODInfo& Info = Component->LODInfo[Index];
												Info.OverrideProfileSkinWeights = Buffer;
											}
										}

										Component->UpdateSkinWeightOverrideBuffer();
									}
								}
							};

						// Put in a skin weight profile request
						if (FSkinWeightProfileManager* Manager = FSkinWeightProfileManager::Get(GetWorld()))
						{
							if (bSkinWeightProfilePending)
							{
								Manager->CancelSkinWeightProfileRequest(this);	
							}

							Manager->RequestSkinWeightProfileStack(NormalizedProfileStack, GetSkinnedAsset(), this, Callback);
							bSkinWeightProfilePending = true;
						}
					}
				}
			}
		}
	}

	if (NormalizedProfileStack.IsEmpty())
	{
		bSkinWeightProfileBufferOverridePending = false;
		bSkinWeightProfilePending = false;
		bSkinWeightProfileSet = false;
	}

	// Store the current layers.
	NormalizedProfileStack.CopyIntoArray(CurrentSkinWeightProfileLayers);

	return bChanged;
}

void USkinnedMeshComponent::ReleaseUpdateRateParams()
{
	FAnimUpdateRateManager::CleanupUpdateRateParametersRef(this);
	AnimUpdateRateParams = nullptr;
}

void USkinnedMeshComponent::RefreshUpdateRateParams()
{
	if (AnimUpdateRateParams)
	{
		ReleaseUpdateRateParams();
	}

	AnimUpdateRateParams = FAnimUpdateRateManager::GetUpdateRateParameters(this);
}

void USkinnedMeshComponent::SetRenderStatic(bool bNewValue)
{
	if (bRenderStatic != bNewValue)
	{
		bRenderStatic = bNewValue;
		PrecachePSOs();
		MarkRenderStateDirty();
	}
}

#if WITH_EDITOR

bool USkinnedMeshComponent::IsCompiling() const
{
	return GetSkinnedAsset() && GetSkinnedAsset()->IsCompiling();
}

void USkinnedMeshComponent::BindWorldDelegates()
{
	FWorldDelegates::OnPostWorldCreation.AddStatic(&HandlePostWorldCreation);
}

void USkinnedMeshComponent::HandlePostWorldCreation(UWorld* InWorld)
{
	TWeakObjectPtr<UWorld> WeakWorld = InWorld;
	InWorld->AddOnFeatureLevelChangedHandler(FOnFeatureLevelChanged::FDelegate::CreateStatic(&HandleFeatureLevelChanged, WeakWorld));
}

void USkinnedMeshComponent::HandleFeatureLevelChanged(ERHIFeatureLevel::Type InFeatureLevel, TWeakObjectPtr<UWorld> InWorld)
{
	if(UWorld* World = InWorld.Get())
	{
		for(TObjectIterator<USkinnedMeshComponent> It; It; ++It)
		{
			if(USkinnedMeshComponent* Component = *It)
			{
				if(Component->GetWorld() == World)
				{
					Component->CachedSceneFeatureLevel = InFeatureLevel;
				}
			}
		}
	}
}
#endif

/** Takes sorted array Base and then adds any elements from sorted array Insert which is missing from it, preserving order.
 * this assumes both arrays are sorted and contain unique bone indices. */
/*static*/ void USkinnedMeshComponent::MergeInBoneIndexArrays(TArray<FBoneIndexType>& BaseArray, const TArray<FBoneIndexType>& InsertArray)
{
	// Then we merge them into the array of required bones.
	int32 BaseBonePos = 0;
	int32 InsertBonePos = 0;

	// Iterate over each of the bones we need.
	while (InsertBonePos < InsertArray.Num())
	{
		// Find index of physics bone
		FBoneIndexType InsertBoneIndex = InsertArray[InsertBonePos];

		// If at end of BaseArray array - just append.
		if (BaseBonePos == BaseArray.Num())
		{
			BaseArray.Add(InsertBoneIndex);
			BaseBonePos++;
			InsertBonePos++;
		}
		// If in the middle of BaseArray, merge together.
		else
		{
			// Check that the BaseArray array is strictly increasing, otherwise merge code does not work.
			check(BaseBonePos == 0 || BaseArray[BaseBonePos - 1] < BaseArray[BaseBonePos]);

			// Get next required bone index.
			FBoneIndexType BaseBoneIndex = BaseArray[BaseBonePos];

			// We have a bone in BaseArray not required by Insert. Thats ok - skip.
			if (BaseBoneIndex < InsertBoneIndex)
			{
				BaseBonePos++;
			}
			// Bone required by Insert is in 
			else if (BaseBoneIndex == InsertBoneIndex)
			{
				BaseBonePos++;
				InsertBonePos++;
			}
			// Bone required by Insert is missing - insert it now.
			else // BaseBoneIndex > InsertBoneIndex
			{
				BaseArray.InsertUninitialized(BaseBonePos);
				BaseArray[BaseBonePos] = InsertBoneIndex;

				BaseBonePos++;
				InsertBonePos++;
			}
		}
	}
}

void USkinnedMeshComponent::GetPhysicsRequiredBones(const USkinnedAsset* SkinnedAsset, const UPhysicsAsset* PhysicsAsset, TArray<FBoneIndexType>& OutRequiredBones)
{
	check(SkinnedAsset != nullptr);
	check(PhysicsAsset != nullptr);

	// If we have a PhysicsAsset, we also need to make sure that all the bones used by it are always updated, as its used
	// by line checks etc. We might also want to kick in the physics, which means having valid bone transforms.
	TArray<FBoneIndexType> PhysAssetBones;
	PhysAssetBones.Reserve(PhysicsAsset->SkeletalBodySetups.Num());
	for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		if (!ensure(PhysicsAsset->SkeletalBodySetups[i]))
		{
			continue;
		}
		const int32 PhysBoneIndex = SkinnedAsset->GetRefSkeleton().FindBoneIndex(PhysicsAsset->SkeletalBodySetups[i]->BoneName);
		if (PhysBoneIndex != INDEX_NONE)
		{
			PhysAssetBones.Add(PhysBoneIndex);
		}
	}

	// Then sort array of required bones in hierarchy order
	PhysAssetBones.Sort();

	// Make sure all of these are in RequiredBones.
	MergeInBoneIndexArrays(OutRequiredBones, PhysAssetBones);
}

void USkinnedMeshComponent::GetSocketRequiredBones(const USkinnedAsset* SkinnedAsset, TArray<FBoneIndexType>& OutRequiredBones, TArray<FBoneIndexType>& NeededBonesForFillComponentSpaceTransforms)
{
	check(SkinnedAsset != nullptr);

	TArray<FBoneIndexType> ForceAnimatedSocketBones;
	const TArray<USkeletalMeshSocket*> ActiveSocketList = SkinnedAsset->GetActiveSocketList();
	ForceAnimatedSocketBones.Reserve(ActiveSocketList.Num());
	for (const USkeletalMeshSocket* Socket : ActiveSocketList)
	{
		const int32 BoneIndex = SkinnedAsset->GetRefSkeleton().FindBoneIndex(Socket->BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			if (Socket->bForceAlwaysAnimated)
			{
				ForceAnimatedSocketBones.AddUnique(BoneIndex);
			}
			else
			{
				NeededBonesForFillComponentSpaceTransforms.AddUnique(BoneIndex);
			}
		}
	}

	// Then sort array of required bones in hierarchy order
	ForceAnimatedSocketBones.Sort();

	// Make sure all of these are in OutRequiredBones.
	MergeInBoneIndexArrays(OutRequiredBones, ForceAnimatedSocketBones);
}

void USkinnedMeshComponent::NotifyIfSkinnedAssetChanged()
{
#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	USkinnedAsset* CurrentSkinnedAsset = !SkeletalMesh ? SkinnedAsset : SkeletalMesh;	// Return the SkeletalMesh pointer for backward compatibility until it is retired
	if (KnownSkinnedAsset != CurrentSkinnedAsset)
	{
		KnownSkinnedAsset = CurrentSkinnedAsset;
		FObjectCacheEventSink::NotifySkinnedMeshChanged_Concurrent(GetSkinnedMeshComponentInterface());
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

#if WITH_EDITOR
void USkinnedMeshComponent::OutdatedKnownSkinnedAssetDetected() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	USkinnedAsset* CurrentSkinnedAsset = !SkeletalMesh ? SkinnedAsset : SkeletalMesh;	// Return the SkeletalMesh pointer for backward compatibility until it is retired
	ensureMsgf(
		KnownSkinnedAsset == CurrentSkinnedAsset,
		TEXT("SkinnedAsset property overwritten for component %s without a call to NotifyIfSkinnedAssetChanged(). KnownSkinnedAsset (%p) != SkinnedAsset (%p - %s)"),
		*GetFullName(),
		KnownSkinnedAsset,
		CurrentSkinnedAsset,
		CurrentSkinnedAsset ? *CurrentSkinnedAsset->GetFullName() : TEXT("nullptr")
		);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// This is a last resort, call the notification now
	USkinnedMeshComponent* MutableThis = const_cast<USkinnedMeshComponent*>(this);
	MutableThis->NotifyIfSkinnedAssetChanged();
}
#endif

void FAnimUpdateRateParameters::SetTrailMode(float DeltaTime, uint8 UpdateRateShift, int32 NewUpdateRate, int32 NewEvaluationRate, bool bNewInterpSkippedFrames)
{
	OptimizeMode = TrailMode;
	ThisTickDelta = DeltaTime;

	UpdateRate = FMath::Max(NewUpdateRate, 1);

	// Make sure EvaluationRate is a multiple of UpdateRate.
	EvaluationRate = FMath::Max((NewEvaluationRate / UpdateRate) * UpdateRate, 1);
	bInterpolateSkippedFrames = (FAnimUpdateRateManager::CVarURODisableInterpolation.GetValueOnAnyThread() == 0) &&
		((bNewInterpSkippedFrames && (EvaluationRate < MaxEvalRateForInterpolation)) || (FAnimUpdateRateManager::CVarForceInterpolation.GetValueOnAnyThread() == 1));

	// Make sure we don't overflow. we don't need very large numbers.
	const uint32 Counter = (GFrameCounter + UpdateRateShift) % MAX_uint32;

	bSkipUpdate = ((Counter % UpdateRate) > 0);
	bSkipEvaluation = ((Counter % EvaluationRate) > 0);

	// As UpdateRate changes, because of LODs for example,
	// make sure we're not caught in a loop where we don't update longer than our update rate.
	{
		SkippedUpdateFrames = bSkipUpdate ? ++SkippedUpdateFrames : 0;
		SkippedEvalFrames = bSkipEvaluation ? ++SkippedEvalFrames : 0;

		// If we've gone longer that our UpdateRate, force an update to happen.
		if ((SkippedUpdateFrames >= UpdateRate) || (SkippedEvalFrames >= EvaluationRate))
		{
			bSkipUpdate = false;
			bSkipEvaluation = false;
			SkippedUpdateFrames = 0;
			SkippedEvalFrames = 0;
		}
	}

	// We should never trigger an Eval without an Update.
	check((bSkipEvaluation && bSkipUpdate) || (bSkipEvaluation && !bSkipUpdate) || (!bSkipEvaluation && !bSkipUpdate));

	AdditionalTime = 0.f;

	if (bSkipUpdate)
	{
		TickedPoseOffestTime -= DeltaTime;
	}
	else
	{
		if (TickedPoseOffestTime < 0.f)
		{
			AdditionalTime = -TickedPoseOffestTime;
			TickedPoseOffestTime = 0.f;
		}
	}
}

void FAnimUpdateRateParameters::SetLookAheadMode(float DeltaTime, uint8 UpdateRateShift, float LookAheadAmount)
{
	float OriginalTickedPoseOffestTime = TickedPoseOffestTime;
	if (OptimizeMode == TrailMode)
	{
		TickedPoseOffestTime = 0.f;
	}
	OptimizeMode = LookAheadMode;
	ThisTickDelta = DeltaTime;

	bInterpolateSkippedFrames = true;

	TickedPoseOffestTime -= DeltaTime;

	if (TickedPoseOffestTime < 0.f)
	{
		LookAheadAmount = FMath::Max(TickedPoseOffestTime*-1.f, LookAheadAmount);
		AdditionalTime = LookAheadAmount;
		TickedPoseOffestTime += LookAheadAmount;

		bool bValid = (TickedPoseOffestTime >= 0.f);
		if (!bValid)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("TPO Time: %.3f | Orig TPO Time: %.3f | DT: %.3f | LookAheadAmount: %.3f\n"), TickedPoseOffestTime, OriginalTickedPoseOffestTime, DeltaTime, LookAheadAmount);
		}
		check(bValid);
		bSkipUpdate = bSkipEvaluation = false;
	}
	else
	{
		AdditionalTime = 0.f;
		bSkipUpdate = bSkipEvaluation = true;
	}
}

float FAnimUpdateRateParameters::GetInterpolationAlpha() const
{
	if (OptimizeMode == TrailMode)
	{
		const float FramesToNextEval = EvaluationRate - SkippedEvalFrames;
		return (EvaluationRate > 1.f) ? FMath::Clamp(1.0f / FramesToNextEval, 0.0f, 1.0f) : 1.f;

	}
	else if (OptimizeMode == LookAheadMode)
	{
		return FMath::Clamp(ThisTickDelta / (TickedPoseOffestTime + ThisTickDelta), 0.f, 1.f);
	}
	check(false); // Unknown mode
	return 0.f;
}

float FAnimUpdateRateParameters::GetRootMotionInterp() const
{
	if (OptimizeMode == LookAheadMode)
	{
		return FMath::Clamp(ThisTickDelta / (TickedPoseOffestTime + ThisTickDelta), 0.f, 1.f);
	}
	return 1.f;
}

/** Simple, CPU evaluation of a vertex's skinned position helper function */
void GetTypedSkinnedTangentBasis(
	const USkinnedMeshComponent* SkinnedComp,
	const FSkelMeshRenderSection& Section,
	const FStaticMeshVertexBuffers& StaticVertexBuffers,
	const FSkinWeightVertexBuffer& SkinWeightVertexBuffer,
	const int32 VertIndex,
	const TArray<FMatrix44f> & RefToLocals,
	FVector3f& OutTangentX,
	FVector3f& OutTangentY,
	FVector3f& OutTangentZ
)
{
	OutTangentX = FVector3f::ZeroVector;
	OutTangentY = FVector3f::ZeroVector;
	OutTangentZ = FVector3f::ZeroVector;

	const USkinnedMeshComponent* const LeaderPoseComponentInst = SkinnedComp->LeaderPoseComponent.Get();
	const USkinnedMeshComponent* BaseComponent = LeaderPoseComponentInst ? LeaderPoseComponentInst : SkinnedComp;

	// Do soft skinning for this vertex.
	const int32 BufferVertIndex = Section.GetVertexBufferIndex() + VertIndex;
	const int32 MaxBoneInfluences = SkinWeightVertexBuffer.GetMaxBoneInfluences();

	const FVector3f VertexTangentX = StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentX(BufferVertIndex);
	const FVector3f VertexTangentY = StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentY(BufferVertIndex);
	const FVector3f VertexTangentZ = StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(BufferVertIndex);

#if !PLATFORM_LITTLE_ENDIAN
	// uint8[] elements in LOD.VertexBufferGPUSkin have been swapped for VET_UBYTE4 vertex stream use
	for (int32 InfluenceIndex = MAX_INFLUENCES - 1; InfluenceIndex >= MAX_INFLUENCES - MaxBoneInfluences; InfluenceIndex--)
#else
	for (int32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; InfluenceIndex++)
#endif
	{
		const int32 MeshBoneIndex = Section.BoneMap[SkinWeightVertexBuffer.GetBoneIndex(BufferVertIndex, InfluenceIndex)];
		const float	Weight = (float)SkinWeightVertexBuffer.GetBoneWeight(BufferVertIndex, InfluenceIndex) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
		const FMatrix44f& RefToLocal = RefToLocals[MeshBoneIndex];
		OutTangentX += RefToLocal.TransformVector(VertexTangentX) * Weight;
		OutTangentY += RefToLocal.TransformVector(VertexTangentY) * Weight;
		OutTangentZ += RefToLocal.TransformVector(VertexTangentZ) * Weight;
	}
}

/** Simple, CPU evaluation of a vertex's skinned position helper function */
template <bool bCachedMatrices>
FVector3f GetTypedSkinnedVertexPosition(
	const USkinnedMeshComponent* SkinnedComp,
	const FSkelMeshRenderSection& Section,
	const FPositionVertexBuffer& PositionVertexBuffer,
	const FSkinWeightVertexBuffer& SkinWeightVertexBuffer,
	const int32 VertIndex,
	const TArray<FMatrix44f> & RefToLocals
)
{
	FVector3f SkinnedPos(0, 0, 0);

	const USkinnedMeshComponent* const LeaderPoseComponentInst = SkinnedComp->LeaderPoseComponent.Get();
	const USkinnedMeshComponent* BaseComponent = LeaderPoseComponentInst ? LeaderPoseComponentInst : SkinnedComp;

	// Do soft skinning for this vertex.
	int32 BufferVertIndex = Section.GetVertexBufferIndex() + VertIndex;
	int32 MaxBoneInfluences = SkinWeightVertexBuffer.GetMaxBoneInfluences();

#if !PLATFORM_LITTLE_ENDIAN
	// uint8[] elements in LOD.VertexBufferGPUSkin have been swapped for VET_UBYTE4 vertex stream use
	for (int32 InfluenceIndex = MAX_INFLUENCES - 1; InfluenceIndex >= MAX_INFLUENCES - MaxBoneInfluences; InfluenceIndex--)
#else
	for (int32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; InfluenceIndex++)
#endif
	{
		const int32 BoneMapIndex = SkinWeightVertexBuffer.GetBoneIndex(BufferVertIndex, InfluenceIndex);
		if (!ensureMsgf(Section.BoneMap.IsValidIndex(BoneMapIndex), TEXT("%s has attempted to access a BoneMap of size %i with an invalid index of %i in GetTypedSkinnedVertexPosition()"), *SkinnedComp->GetSkinnedAsset()->GetFullName(), Section.BoneMap.Num(), BoneMapIndex))
		{
			continue;
		}

		const int32 MeshBoneIndex = Section.BoneMap[BoneMapIndex];
		int32 TransformBoneIndex = MeshBoneIndex;

		if (LeaderPoseComponentInst)
		{
			const TArray<int32>& LeaderBoneMap = SkinnedComp->GetLeaderBoneMap();
			check(LeaderBoneMap.Num() == SkinnedComp->GetSkinnedAsset()->GetRefSkeleton().GetNum());
			TransformBoneIndex = LeaderBoneMap[MeshBoneIndex];
		}

		const float	Weight = (float)SkinWeightVertexBuffer.GetBoneWeight(BufferVertIndex, InfluenceIndex) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
		{
			if (bCachedMatrices)
			{
				if (ensureMsgf(RefToLocals.IsValidIndex(MeshBoneIndex), TEXT("%s has attempted to access a RefToLocals of size %i with an invalid index of %i in GetTypedSkinnedVertexPosition()"), *SkinnedComp->GetSkinnedAsset()->GetFullName(), RefToLocals.Num(), MeshBoneIndex))
				{
					const FMatrix44f& RefToLocal = RefToLocals[MeshBoneIndex];
					SkinnedPos += RefToLocal.TransformPosition(PositionVertexBuffer.VertexPosition(BufferVertIndex)) * Weight;
				}
			}
			else
			{
				const FMatrix44f BoneTransformMatrix = (TransformBoneIndex != INDEX_NONE) ? (FMatrix44f)BaseComponent->GetComponentSpaceTransforms()[TransformBoneIndex].ToMatrixWithScale() : FMatrix44f::Identity;
				const FMatrix44f RefToLocal = SkinnedComp->GetSkinnedAsset()->GetRefBasesInvMatrix()[MeshBoneIndex] * BoneTransformMatrix;
				SkinnedPos += RefToLocal.TransformPosition(PositionVertexBuffer.VertexPosition(BufferVertIndex)) * Weight;
			}
		}
	}

	return SkinnedPos;
}



template FVector3f GetTypedSkinnedVertexPosition<true>(const USkinnedMeshComponent* SkinnedComp, const FSkelMeshRenderSection& Section, const FPositionVertexBuffer& PositionVertexBuffer,
	const FSkinWeightVertexBuffer& SkinWeightVertexBuffer, const int32 VertIndex, const TArray<FMatrix44f> & RefToLocals);

template FVector3f GetTypedSkinnedVertexPosition<false>(const USkinnedMeshComponent* SkinnedComp, const FSkelMeshRenderSection& Section, const FPositionVertexBuffer& PositionVertexBuffer,
	const FSkinWeightVertexBuffer& SkinWeightVertexBuffer, const int32 VertIndex, const TArray<FMatrix44f> & RefToLocals);

#undef LOCTEXT_NAMESPACE
