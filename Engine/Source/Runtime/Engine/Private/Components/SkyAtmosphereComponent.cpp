// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SkyAtmosphereComponent.h"

#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Engine/Level.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "MeshElementCollector.h"
#include "SceneInterface.h"
#include "SceneProxies/SkyAtmosphereSceneProxy.h"
#include "StateStream/SkyAtmosphereStateStream.h"
#include "UObject/UObjectIterator.h"
#include "PrimitiveDrawingUtils.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/DirectionalLightComponent.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "ColorManagement/ColorSpace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkyAtmosphereComponent)

#if WITH_EDITOR
#include "ObjectEditorUtils.h"
#include "Rendering/StaticLightingSystemInterface.h"
#endif

#define LOCTEXT_NAMESPACE "SkyAtmosphereComponent"

#if WITH_STATE_STREAM_ACTOR
constexpr bool UseSkyAtmosphereStateStream = true;
#endif

DEFINE_LOG_CATEGORY(LogSkyAtmosphereComponent)


/*=============================================================================
	USkyAtmosphereComponent implementation.
=============================================================================*/

//#define USKYATMOSPHERE_VARIABLE(Type, Owner, Dot, Prefix, Name)

#define USKYATMOSPHERE_VARIABLES \
		USKYATMOSPHERE_VARIABLE(ESkyAtmosphereTransformMode, , , , TransformMode) \
		USKYATMOSPHERE_VARIABLE(float, , , , TraceSampleCountScale) \
		USKYATMOSPHERE_VARIABLE(float, OtherTentDistribution, ., , TipAltitude) \
		USKYATMOSPHERE_VARIABLE(float, OtherTentDistribution, ., , TipValue) \
		USKYATMOSPHERE_VARIABLE(float, OtherTentDistribution, ., , Width) \
		USKYATMOSPHERE_VARIABLE(uint8, , , b, Holdout) \
		USKYATMOSPHERE_VARIABLE(uint8, , , b, RenderInMainPass) \
		USKYATMOSPHERE_VARIABLES_WITH_SETFUNCTION

#define USKYATMOSPHERE_VARIABLES_WITH_SETFUNCTION \
		USKYATMOSPHERE_VARIABLE(float, , , , BottomRadius) \
		USKYATMOSPHERE_VARIABLE(const FColor&, , , , GroundAlbedo) \
		USKYATMOSPHERE_VARIABLE(float, , , , AtmosphereHeight) \
		USKYATMOSPHERE_VARIABLE(float, , , , MultiScatteringFactor) \
		USKYATMOSPHERE_VARIABLE(float, , , , RayleighScatteringScale) \
		USKYATMOSPHERE_VARIABLE(FLinearColor, , , , RayleighScattering) \
		USKYATMOSPHERE_VARIABLE(float, , , , RayleighExponentialDistribution) \
		USKYATMOSPHERE_VARIABLE(float, , , , MieScatteringScale) \
		USKYATMOSPHERE_VARIABLE(FLinearColor, , , , MieScattering) \
		USKYATMOSPHERE_VARIABLE(float, , , , MieAbsorptionScale) \
		USKYATMOSPHERE_VARIABLE(FLinearColor, , , , MieAbsorption) \
		USKYATMOSPHERE_VARIABLE(float, , , , MieAnisotropy) \
		USKYATMOSPHERE_VARIABLE(float, , , , MieExponentialDistribution) \
		USKYATMOSPHERE_VARIABLE(float, , , , OtherAbsorptionScale) \
		USKYATMOSPHERE_VARIABLE(FLinearColor, , , , OtherAbsorption) \
		USKYATMOSPHERE_VARIABLE(FLinearColor, , , , SkyLuminanceFactor) \
		USKYATMOSPHERE_VARIABLE(FLinearColor, , , , SkyAndAerialPerspectiveLuminanceFactor) \
		USKYATMOSPHERE_VARIABLE(float, , , , AerialPespectiveViewDistanceScale) \
		USKYATMOSPHERE_VARIABLE(float, , , , AerialPerspectiveStartDepth) \
		USKYATMOSPHERE_VARIABLE(float, , , , HeightFogContribution) \
		USKYATMOSPHERE_VARIABLE(float, , , , TransmittanceMinLightElevationAngle) \

FOverrideAtmosphericLight ConvertAtmosphericLightOverride(bool* Enabled, FVector* Direction)
{
	FOverrideAtmosphericLight Override;
	for (uint32 I=0; I!=NUM_ATMOSPHERE_LIGHTS; ++I)
	{
		Override.EnabledMask |= uint8(Enabled[I]) << I;
		Override.Direction[I] = Direction[I];
	}
	return Override;
}

USkyAtmosphereComponent::USkyAtmosphereComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SkyAtmosphereSceneProxy(nullptr)
{
	// All distance here are in kilometer and scattering/absorptions coefficient in 1/kilometers.
	const float EarthBottomRadius = 6360.0f;
	const float EarthTopRadius = 6420.0f;
	const float EarthRayleighScaleHeight = 8.0f;
	const float EarthMieScaleHeight = 1.2f;
	
	// Default: Earth like atmosphere
	TransformMode = ESkyAtmosphereTransformMode::PlanetTopAtAbsoluteWorldOrigin;
	BottomRadius = EarthBottomRadius;
	AtmosphereHeight = EarthTopRadius - EarthBottomRadius;
	GroundAlbedo = FColor(170, 170, 170); // 170 => 0.4f linear

	// Float to a u8 rgb + float length can lose some precision but it is better UI wise.
	const FLinearColor RayleightScatteringRaw = FLinearColor(0.005802f, 0.013558f, 0.033100f);
	RayleighScattering = RayleightScatteringRaw * (1.0f / RayleightScatteringRaw.B);
	RayleighScatteringScale = RayleightScatteringRaw.B;
	RayleighExponentialDistribution = EarthRayleighScaleHeight;

	MieScattering = FColor::White;
	MieScatteringScale = 0.003996f;
	MieAbsorption = FColor::White;
	MieAbsorptionScale = 0.000444f;
	MieAnisotropy = 0.8f;
	MieExponentialDistribution = EarthMieScaleHeight;

	// Absorption tent distribution representing ozone distribution in Earth atmosphere.
	const FLinearColor OtherAbsorptionRaw = FLinearColor(0.000650f, 0.001881f, 0.000085f);
	OtherAbsorptionScale = OtherAbsorptionRaw.G;
	OtherAbsorption = OtherAbsorptionRaw * (1.0f / OtherAbsorptionRaw.G);
	OtherTentDistribution.TipAltitude = 25.0f;
	OtherTentDistribution.TipValue    =  1.0f;
	OtherTentDistribution.Width       = 15.0f;

	SkyLuminanceFactor = FLinearColor(FLinearColor::White);
	SkyAndAerialPerspectiveLuminanceFactor = FLinearColor(FLinearColor::White);
	MultiScatteringFactor = 1.0f;
	AerialPespectiveViewDistanceScale = 1.0f;
	HeightFogContribution = 1.0f;
	TransmittanceMinLightElevationAngle = -90.0f;
	AerialPerspectiveStartDepth = 0.1f;

	TraceSampleCountScale = 1.0f;

	bHoldout = false;
	bRenderInMainPass = true;

	memset(OverrideAtmosphericLight, 0, sizeof(OverrideAtmosphericLight));

	if(!IsTemplate())
	{
		ValidateStaticLightingGUIDs();
	}
}

USkyAtmosphereComponent::~USkyAtmosphereComponent()
{
}

static bool SkyAtmosphereComponentStaticLightingBuilt(const USkyAtmosphereComponent* Component)
{
	AActor* Owner = Component->GetOwner();
	UMapBuildDataRegistry* Registry = nullptr;
	if (Owner)
	{
		ULevel* OwnerLevel = Owner->GetLevel();
		if (OwnerLevel && OwnerLevel->OwningWorld)
		{
			ULevel* ActiveLightingScenario = OwnerLevel->OwningWorld->GetActiveLightingScenario();
			if (ActiveLightingScenario && ActiveLightingScenario->MapBuildData)
			{
				Registry = ActiveLightingScenario->MapBuildData;
			}
			else if (OwnerLevel->MapBuildData)
			{
				Registry = OwnerLevel->MapBuildData;
			}
		}
	}

	const FSkyAtmosphereMapBuildData* SkyAtmosphereFogBuildData = Registry ? Registry->GetSkyAtmosphereBuildData(Component->GetStaticLightingBuiltGuid()) : nullptr;
	UWorld* World = Component->GetWorld();
	if (World)
	{
		class FSceneInterface* Scene = Component->GetWorld()->Scene;

		// Only require building if there is a Sky or Sun light requiring lighting builds, i.e. non movable.
		const bool StaticLightingDependsOnAtmosphere = Scene->HasSkyLightRequiringLightingBuild() || Scene->HasAtmosphereLightRequiringLightingBuild();
		// Built data is available or static lighting does not depend any sun/sky components.
		return (SkyAtmosphereFogBuildData != nullptr && StaticLightingDependsOnAtmosphere) || !StaticLightingDependsOnAtmosphere;
	}

	return true;	// The component has not been spawned in any world yet so let's mark it as built for now.
}

void USkyAtmosphereComponent::SendRenderTransformCommand()
{
	if (SkyAtmosphereSceneProxy)
	{
		FTransform ComponentTransform = GetComponentTransform();
		uint8 TrsfMode = uint8(TransformMode);
		FSkyAtmosphereSceneProxy* SceneProxy = SkyAtmosphereSceneProxy;
		ENQUEUE_RENDER_COMMAND(FUpdateSkyAtmosphereSceneProxyTransformCommand)(
			[SceneProxy, ComponentTransform, TrsfMode](FRHICommandList& RHICmdList)
		{
			SceneProxy->UpdateTransform(ComponentTransform, TrsfMode);
		});
	}
}

void USkyAtmosphereComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);
	// If one day we need to look up lightmass built data, lookup it up here using the guid from the correct MapBuildData.

	bool bHidden = false;
#if WITH_EDITORONLY_DATA
	bHidden = GetOwner() ? GetOwner()->bHiddenEdLevel : false;
#endif // WITH_EDITORONLY_DATA
	if (!ShouldComponentAddToScene())
	{
		bHidden = true;
	}

	if (GetVisibleFlag() && !bHidden &&
		ShouldComponentAddToScene() && ShouldRender() && IsRegistered() && (GetOuter() == NULL || !GetOuter()->HasAnyFlags(RF_ClassDefaultObject)))
	{
		bool bBuilt = SkyAtmosphereComponentStaticLightingBuilt(this);
		#if WITH_STATE_STREAM_ACTOR
		if (UseSkyAtmosphereStateStream)
		{
			FSkyAtmosphereDynamicState Ds;
			#define USKYATMOSPHERE_VARIABLE(Type, Owner, Dot, Prefix, Name) Ds.Set##Owner##Name(Owner Dot Prefix##Name);
			USKYATMOSPHERE_VARIABLES
			#undef USKYATMOSPHERE_VARIABLE
			Ds.SetBuilt(bBuilt);
			Ds.SetComponentTransform(GetComponentTransform());
			Ds.SetOverrideAtmosphericLight(ConvertAtmosphericLightOverride(OverrideAtmosphericLight, OverrideAtmosphericLightDirection));
			Handle = GetWorld()->GetStateStream<ISkyAtmosphereStateStream>().Game_CreateInstance({}, Ds);
			return;
		}
		#endif

		// Create the scene proxy.
		SkyAtmosphereSceneProxy = new FSkyAtmosphereSceneProxy(this);
		GetWorld()->Scene->AddSkyAtmosphere(SkyAtmosphereSceneProxy, bBuilt);
	}

}

void USkyAtmosphereComponent::SendRenderTransform_Concurrent()
{
	Super::SendRenderTransform_Concurrent();
	SendRenderTransformCommand();
}

void USkyAtmosphereComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	#if WITH_STATE_STREAM_ACTOR
	Handle = {};
	#endif

	if (SkyAtmosphereSceneProxy)
	{
		GetWorld()->Scene->RemoveSkyAtmosphere(SkyAtmosphereSceneProxy);

		FSkyAtmosphereSceneProxy* SceneProxy = SkyAtmosphereSceneProxy;
		ENQUEUE_RENDER_COMMAND(FDestroySkyAtmosphereSceneProxyCommand)(
			[SceneProxy](FRHICommandList& RHICmdList)
		{
			delete SceneProxy;
		});

		SkyAtmosphereSceneProxy = nullptr;
	}
}

void USkyAtmosphereComponent::ValidateStaticLightingGUIDs()
{
	// Validate light guids.
	if (!bStaticLightingBuiltGUID.IsValid())
	{
		UpdateStaticLightingGUIDs();
	}
}

void USkyAtmosphereComponent::UpdateStaticLightingGUIDs()
{
	bStaticLightingBuiltGUID = FGuid::NewGuid();
}

void USkyAtmosphereComponent::SetDummyStaticLightingGUIDs()
{
	// Dummy GUID just to make sure the value is initialized and not random.
	bStaticLightingBuiltGUID = FGuid(1, 0, 0, 0);
}

#if WITH_EDITOR

void USkyAtmosphereComponent::CheckForErrors()
{
	AActor* Owner = GetOwner();
	if (Owner && GetVisibleFlag())
	{
		UWorld* ThisWorld = Owner->GetWorld();
		bool bMultipleFound = false;

		if (ThisWorld)
		{
			for (TObjectIterator<USkyAtmosphereComponent> ComponentIt; ComponentIt; ++ComponentIt)
			{
				USkyAtmosphereComponent* Component = *ComponentIt;

				if (Component != this
					&& IsValid(Component)
					&& Component->GetVisibleFlag()
					&& Component->GetOwner()
					&& ThisWorld->ContainsActor(Component->GetOwner())
					&& IsValid(Component->GetOwner()))
				{
					bMultipleFound = true;
					break;
				}
			}
		}

		if (bMultipleFound)
		{
			FMessageLog("MapCheck").Error()
				->AddToken(FUObjectToken::Create(Owner))
				->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_MultipleSkyAtmosphere", "Multiple sky atmosphere are active, only one can be enabled per world.")))
				->AddToken(FMapErrorToken::Create(FMapErrors::MultipleSkyAtmospheres));
		}
	}
}

void USkyAtmosphereComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// If any properties have been changed in the atmosphere category, it means the sky look will change and lighting needs to be rebuild.
	const FName CategoryName = FObjectEditorUtils::GetCategoryFName(PropertyChangedEvent.Property);
	if (CategoryName == FName("Planet") ||
		CategoryName == FName("Atmosphere") ||
		CategoryName == FName("Atmosphere - Rayleigh") ||
		CategoryName == FName("Atmosphere - Mie") ||
		CategoryName == FName("Atmosphere - Absorption") ||
		CategoryName == FName("Art direction"))
	{
		if (SkyAtmosphereComponentStaticLightingBuilt(this))
		{
			// If we have changed an atmosphere property and the lighting has already been built, we need to ask for a rebuild by updating the static lighting GUIDs.
			UpdateStaticLightingGUIDs();
		}

		if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, TransformMode))
		{
			SendRenderTransformCommand();
		}

#if WITH_EDITOR
		FStaticLightingSystemInterface::OnSkyAtmosphereModified.Broadcast();
#endif

	}
}

#endif // WITH_EDITOR

void USkyAtmosphereComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	// Only load the lighting GUID if
	if( (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::RemovedAtmosphericFog && Ar.IsLoading() && bIsAtmosphericFog) //Loading an AtmosphereFog component into a SkyAtmosphere component
		|| (Ar.IsSaving() && bIsAtmosphericFog)	// Saving an AtmosphereFog component as a SkyAtmosphere component
		|| !bIsAtmosphericFog) // Saving / Loading a regular SkyAtmosphere
	{
		// Only load that for SkyAtmosphere or AtmosphericFog component that have already been converted
		Ar << bStaticLightingBuiltGUID;
	}
}

void USkyAtmosphereComponent::OverrideAtmosphereLightDirection(int32 AtmosphereLightIndex, const FVector& LightDirection)
{
	check(AtmosphereLightIndex >= 0 && AtmosphereLightIndex < NUM_ATMOSPHERE_LIGHTS);
	if (AreDynamicDataChangesAllowed() && SkyAtmosphereSceneProxy &&
		(!OverrideAtmosphericLight[AtmosphereLightIndex] || OverrideAtmosphericLightDirection[AtmosphereLightIndex]!=LightDirection))
	{
		OverrideAtmosphericLight[AtmosphereLightIndex] = true;
		OverrideAtmosphericLightDirection[AtmosphereLightIndex] = LightDirection;

		#if WITH_STATE_STREAM_ACTOR
		if (UseSkyAtmosphereStateStream)
		{
			FSkyAtmosphereDynamicState Ds;
			Ds.SetOverrideAtmosphericLight(ConvertAtmosphericLightOverride(OverrideAtmosphericLight, OverrideAtmosphericLightDirection));
			Handle.Update(Ds);
			return;
		}
		#endif
		MarkRenderStateDirty();
	}
}

bool USkyAtmosphereComponent::IsAtmosphereLightDirectionOverriden(int32 AtmosphereLightIndex)
{
	check(AtmosphereLightIndex >= 0 && AtmosphereLightIndex < NUM_ATMOSPHERE_LIGHTS);
	if (AtmosphereLightIndex >= 0 && AtmosphereLightIndex < NUM_ATMOSPHERE_LIGHTS)
	{
		return OverrideAtmosphericLight[AtmosphereLightIndex];
	}
	return false;
}

FVector USkyAtmosphereComponent::GetOverridenAtmosphereLightDirection(int32 AtmosphereLightIndex)
{
	check(AtmosphereLightIndex >= 0 && AtmosphereLightIndex < NUM_ATMOSPHERE_LIGHTS);
	if (AtmosphereLightIndex >= 0 && AtmosphereLightIndex < NUM_ATMOSPHERE_LIGHTS)
	{
		return OverrideAtmosphericLightDirection[AtmosphereLightIndex];
	}
	return FVector::ZeroVector;
}

void USkyAtmosphereComponent::ResetAtmosphereLightDirectionOverride(int32 AtmosphereLightIndex)
{
	check(AtmosphereLightIndex >= 0 && AtmosphereLightIndex < NUM_ATMOSPHERE_LIGHTS);
	if (AtmosphereLightIndex >= 0 && AtmosphereLightIndex < NUM_ATMOSPHERE_LIGHTS)
	{
		OverrideAtmosphericLight[AtmosphereLightIndex] = false;
		OverrideAtmosphericLightDirection[AtmosphereLightIndex] = FVector::ZeroVector;
	}
}

void USkyAtmosphereComponent::GetOverrideLightStatus(bool* OutOverrideAtmosphericLight, FVector* OutOverrideAtmosphericLightDirection) const
{
	memcpy(OutOverrideAtmosphericLight, OverrideAtmosphericLight, sizeof(OverrideAtmosphericLight));
	memcpy(OutOverrideAtmosphericLightDirection, OverrideAtmosphericLightDirection, sizeof(OverrideAtmosphericLightDirection));
}

void USkyAtmosphereComponent::SetPositionToMatchDeprecatedAtmosphericFog()
{
	TransformMode = ESkyAtmosphereTransformMode::PlanetTopAtComponentTransform;
	SetWorldLocation(FVector(0.0f, 0.0f, -100000.0f));
}

FLinearColor SkyAtmosphereGetClamped(FLinearColor Value) { return Value.GetClamped(0.0f, 1e38f); }
template<typename T> T SkyAtmosphereGetClamped(T Value) { return Value; }

#if WITH_STATE_STREAM_ACTOR
	#define USKYATMOSPHERE_SETHANDLE(Field) \
		if (UseSkyAtmosphereStateStream) \
		{ \
			FSkyAtmosphereDynamicState Ds; \
			Ds.Set##Field(Field); \
			Handle.Update(Ds); \
			return; \
		}
#else
	#define USKYATMOSPHERE_SETHANDLE(Field)
#endif

#define USKYATMOSPHERE_VARIABLE(Type, Owner, Dot, Prefix, Name) \
	void USkyAtmosphereComponent::Set##Name(Type NewValue) \
	{ \
		if (AreDynamicDataChangesAllowed() && Name != NewValue) \
		{ \
			Name = SkyAtmosphereGetClamped(NewValue); \
			USKYATMOSPHERE_SETHANDLE(Name) \
			MarkRenderStateDirty(); \
		} \
	}

USKYATMOSPHERE_VARIABLES_WITH_SETFUNCTION
#undef USKYATMOSPHERE_VARIABLE

void USkyAtmosphereComponent::SetHoldout(bool bNewHoldout)
{
	if (bHoldout != bNewHoldout)
	{
		bHoldout = bNewHoldout;
#if WITH_STATE_STREAM_ACTOR
		if (UseSkyAtmosphereStateStream)
		{
			FSkyAtmosphereDynamicState Ds;
			Ds.SetHoldout(bNewHoldout);
			Handle.Update(Ds);
			return;
		}
#endif
		MarkRenderStateDirty();
	}
}

void USkyAtmosphereComponent::SetRenderInMainPass(bool bValue)
{
	if (bRenderInMainPass != bValue)
	{
		bRenderInMainPass = bValue;
#if WITH_STATE_STREAM_ACTOR
		if (UseSkyAtmosphereStateStream)
		{
			FSkyAtmosphereDynamicState Ds;
			Ds.SetRenderInMainPass(bValue);
			Handle.Update(Ds);
			return;
		}
#endif
		MarkRenderStateDirty();
	}
}

FLinearColor USkyAtmosphereComponent::GetAtmosphereTransmitanceOnGroundAtPlanetTop(UDirectionalLightComponent* DirectionalLight)
{
	if(DirectionalLight != nullptr)
	{
		FAtmosphereSetup AtmosphereSetup(*this);
		const FLinearColor TransmittanceAtDirLight = AtmosphereSetup.GetTransmittanceAtGroundLevel(-DirectionalLight->GetDirection());
		return TransmittanceAtDirLight;
	}
	return FLinearColor::White;
}

float USkyAtmosphereComponent::GetAtmosphericLightToMatchIlluminanceOnGround(FVector LightDirection, float IlluminanceOnGround)
{
	FAtmosphereSetup AtmosphereSetup(*this);
	const FLinearColor TransmittanceAtDirLight = AtmosphereSetup.GetTransmittanceAtGroundLevel(LightDirection);
	const float OuterSpaceIlluminance = IlluminanceOnGround / FMath::Max(UE_SMALL_NUMBER, UE::Color::FColorSpace::GetWorking().GetLuminance(TransmittanceAtDirLight));
	return OuterSpaceIlluminance;
}

/*=============================================================================
	ASkyAtmosphere implementation.
=============================================================================*/

#if WITH_EDITOR
#include "ObjectEditorUtils.h"
#endif

ASkyAtmosphere::ASkyAtmosphere(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SkyAtmosphereComponent = CreateDefaultSubobject<USkyAtmosphereComponent>(TEXT("SkyAtmosphereComponent"));
	RootComponent = SkyAtmosphereComponent;

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SkyAtmosphereTextureObject;
			FName ID_SkyAtmosphere;
			FText NAME_SkyAtmosphere;
			FConstructorStatics()
				: SkyAtmosphereTextureObject(TEXT("/Engine/EditorResources/S_SkyAtmosphere"))
				, ID_SkyAtmosphere(TEXT("Fog"))
				, NAME_SkyAtmosphere(NSLOCTEXT("SpriteCategory", "Fog", "Fog"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (GetSpriteComponent())
		{
			GetSpriteComponent()->Sprite = ConstructorStatics.SkyAtmosphereTextureObject.Get();
			GetSpriteComponent()->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			GetSpriteComponent()->SpriteInfo.Category = ConstructorStatics.ID_SkyAtmosphere;
			GetSpriteComponent()->SpriteInfo.DisplayName = ConstructorStatics.NAME_SkyAtmosphere;
			GetSpriteComponent()->SetupAttachment(SkyAtmosphereComponent);
		}

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(150, 200, 255);

			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_SkyAtmosphere;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_SkyAtmosphere;
			ArrowComponent->SetupAttachment(SkyAtmosphereComponent);
			ArrowComponent->bLightAttachment = true;
			ArrowComponent->bIsScreenSizeScaled = true;
		}
	}
#endif // WITH_EDITORONLY_DATA

	PrimaryActorTick.bCanEverTick = false;
	SetHidden(false);
}



/*=============================================================================
	FSkyAtmosphereSceneProxy implementation.
=============================================================================*/



FSkyAtmosphereSceneProxy::FSkyAtmosphereSceneProxy(const USkyAtmosphereComponent* InComponent)
	: bStaticLightingBuilt(false)
	, RenderSceneInfo(nullptr)
	, AtmosphereSetup(*InComponent)
	, bHoldout(InComponent->bHoldout > 0)
	, bRenderInMainPass(InComponent->bRenderInMainPass > 0)
{
	SkyLuminanceFactor = InComponent->SkyLuminanceFactor;
	SkyAndAerialPerspectiveLuminanceFactor = InComponent->SkyAndAerialPerspectiveLuminanceFactor;
	AerialPespectiveViewDistanceScale = InComponent->AerialPespectiveViewDistanceScale;
	HeightFogContribution = InComponent->HeightFogContribution;
	AerialPerspectiveStartDepthKm = InComponent->AerialPerspectiveStartDepth;
	TraceSampleCountScale = InComponent->TraceSampleCountScale;

	InComponent->GetOverrideLightStatus(OverrideAtmosphericLight, OverrideAtmosphericLightDirection);
}

FSkyAtmosphereSceneProxy::FSkyAtmosphereSceneProxy(const FSkyAtmosphereDynamicState& Ds)
	: bStaticLightingBuilt(false)
	, RenderSceneInfo(nullptr)
	, AtmosphereSetup(Ds)
	, bHoldout(Ds.bHoldout > 0)
	, bRenderInMainPass(Ds.bRenderInMainPass > 0)
{
	SkyLuminanceFactor = Ds.SkyLuminanceFactor;
	SkyAndAerialPerspectiveLuminanceFactor = Ds.SkyAndAerialPerspectiveLuminanceFactor;
	AerialPespectiveViewDistanceScale = Ds.AerialPespectiveViewDistanceScale;
	HeightFogContribution = Ds.HeightFogContribution;
	AerialPerspectiveStartDepthKm = Ds.AerialPerspectiveStartDepth;
	TraceSampleCountScale = Ds.TraceSampleCountScale;

	const FOverrideAtmosphericLight& Overrides = Ds.OverrideAtmosphericLight;
	for (uint32 I=0; I!=NUM_ATMOSPHERE_LIGHTS; ++I)
	{
		OverrideAtmosphericLight[I] = (Overrides.EnabledMask & (1 << I)) != 0;
		OverrideAtmosphericLightDirection[I] = Overrides.Direction[I];
	}
}

FSkyAtmosphereSceneProxy::~FSkyAtmosphereSceneProxy()
{
}

FVector FSkyAtmosphereSceneProxy::GetAtmosphereLightDirection(int32 AtmosphereLightIndex, const FVector& DefaultDirection) const
{
	if (OverrideAtmosphericLight[AtmosphereLightIndex])
	{
		return OverrideAtmosphericLightDirection[AtmosphereLightIndex];
	}
	return DefaultDirection;
}


#undef LOCTEXT_NAMESPACE



