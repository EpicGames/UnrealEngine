// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyLightComponent.cpp: SkyLightComponent implementation.
=============================================================================*/

#include "Components/SkyLightComponent.h"
#include "Engine/Level.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "GameFramework/Info.h"
#include "SceneManagement.h"
#include "Misc/QueuedThreadPool.h"
#include "UObject/ConstructorHelpers.h"
#include "RenderUtils.h"
#include "UObject/UObjectIterator.h"
#include "Engine/SkyLight.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Net/UnrealNetwork.h"
#include "Misc/MapErrors.h"
#include "Misc/ScopeLock.h"
#include "SceneInterface.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "ShaderCompiler.h"
#include "Components/BillboardComponent.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Engine/TextureCube.h"
#include "ComponentRecreateRenderStateContext.h"

#if RHI_RAYTRACING
#endif

#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#include "TextureCompiler.h"
#include "StaticMeshCompiler.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkyLightComponent)

#define LOCTEXT_NAMESPACE "SkyLightComponent"

void OnUpdateSkylights(UWorld* InWorld)
{
	for (TObjectIterator<USkyLightComponent> It; It; ++It)
	{
		USkyLightComponent* SkylightComponent = *It;
		if (InWorld->ContainsActor(SkylightComponent->GetOwner()) && IsValid(SkylightComponent))
		{			
			SkylightComponent->SetCaptureIsDirty();			
		}
	}
	USkyLightComponent::UpdateSkyCaptureContents(InWorld);
}

static bool SkipStaticSkyLightCapture(USkyLightComponent& SkyLight)
{
	// We do the following because capture is a heavy operation that can time out on some platforms at launch. But it is not needed for a static sky light.
	// According to mobility, we remove sky light from capture update queue if Mobility==Static==StaticLighting. The render side proxy will never be created.
	// We do not even need to check if lighting as been built because the skylight does not generate reflection in the static mobility case.
	// and Lightmass will capture the scene in any case independently using CaptureEmissiveRadianceEnvironmentCubeMap.
	// This is also fine in editor because a static sky light will not contribute to any lighting when drag and drop in a level and captured. 
	// In this case only a "lighting build" will result in usable lighting on any objects.
	// One exception however is when ray tracing is enabled as light mobility is not relevant to ray tracing effects, many still requiring information from the sky light even if it is static.
	// Lumen also operates on static skylights and may be enabled when either Ray Tracing or Mesh Distance Fields are supported for the project
	return SkyLight.HasStaticLighting() && !IsRayTracingEnabled() && !DoesProjectSupportDistanceFields();
}

FAutoConsoleCommandWithWorld CaptureConsoleCommand(
	TEXT("r.SkylightRecapture"),
	TEXT("Updates all stationary and movable skylights, useful for debugging the capture pipeline"),
	FConsoleCommandWithWorldDelegate::CreateStatic(OnUpdateSkylights)
	);

int32 GUpdateSkylightsEveryFrame = 0;
FAutoConsoleVariableRef CVarUpdateSkylightsEveryFrame(
	TEXT("r.SkylightUpdateEveryFrame"),
	GUpdateSkylightsEveryFrame,
	TEXT("Whether to update all skylights every frame.  Useful for debugging."),
	ECVF_Default
	);

float GSkylightIntensityMultiplier = 1.0f;
FAutoConsoleVariableRef CVarSkylightIntensityMultiplier(
	TEXT("r.SkylightIntensityMultiplier"),
	GSkylightIntensityMultiplier,
	TEXT("Intensity scale on Stationary and Movable skylights.  This is useful to control overall lighting contrast in dynamically lit games with scalability levels which disable Ambient Occlusion.  For example, if medium quality disables SSAO and DFAO, reduce skylight intensity."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

// The project setting for the sky light component to affect the height fog in scattering (disable runtime and shader code)
static TAutoConsoleVariable<int32> CVarSupportsSkyLightCaptureAffectsHeightFogInScattering(
	TEXT("r.SupportsSkyLightCaptureAffectsHeightFogInScattering"),
	0,
	TEXT("Enables the inscattering of the sky light capture into the distant height fog."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

void OnChangeSkylightRealTimeReflectionCapture(IConsoleVariable* Var)
{
	if (!GIsRHIInitialized)
	{
		return;
	}
	
	// r.SkyLight.RealTimeReflectionCapture is set based on the "Effect" quality level (to be supported, or not, on some different platofrms).
	// When that quality level changes, real-time sky capture can become disabled. In this case, sky light recapture should be scheduled to match the current quality level.
	for (TObjectIterator<USkyLightComponent> It; It; ++It)
	{
		USkyLightComponent* SkylightComponent = *It;
		if (IsValid(SkylightComponent))
		{
			SkylightComponent->SetCaptureIsDirty();
		}
	}

	// We need to recreate the render state of everything after the skylight.
	// This is required for correct update due to recreation order when setting global illumination back from low/medium to high or more.
	FGlobalComponentRecreateRenderStateContext Context;
}

int32 GSkylightRealTimeReflectionCapture = 1;
FAutoConsoleVariableRef CVarSkylightRealTimeReflectionCapture(
	TEXT("r.SkyLight.RealTimeReflectionCapture"),
	GSkylightRealTimeReflectionCapture,
	TEXT("Make sure the sky light real time capture is not run on platform where it is considered out of budget. Cannot be changed at runtime."),
	FConsoleVariableDelegate::CreateStatic(&OnChangeSkylightRealTimeReflectionCapture),
	ECVF_Scalability
	);

int32 GSkylightCubemapMaxResolution = -1;
FAutoConsoleVariableRef CVarSkylightCubemapMaxResolution(
	TEXT("r.SkyLight.CubemapMaxResolution"),
	GSkylightCubemapMaxResolution,
	TEXT("Force max resolution of skylight cubemap (default to -1: takes default property value of USkyLightComponent::CubeMapResolution)")
);

constexpr EPixelFormat SKYLIGHT_CUBEMAP_FORMAT = PF_FloatRGBA;

void FSkyTextureCubeResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (GetFeatureLevel() >= ERHIFeatureLevel::SM5 || GSupportsRenderTargetFormat_PF_FloatRGBA)
	{
		checkf(FMath::IsPowerOfTwo(Size), TEXT("Size of SkyTextureCube must be a power of two; size is %d"), Size);

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::CreateCube(TEXT("SkyTextureCube"), Size, Format)
			.SetNumMips(NumMips);

		TextureCubeRHI = RHICmdList.CreateTexture(Desc);
		TextureRHI = TextureCubeRHI;

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer
		(
			SF_Trilinear,
			AM_Clamp,
			AM_Clamp,
			AM_Clamp
		);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
	}
}

void FSkyTextureCubeResource::Release()
{
	check( IsInGameThread() );
	checkSlow(NumRefs > 0);
	if(--NumRefs == 0)
	{
		BeginReleaseResource(this);
		// Have to defer actual deletion until above rendering command has been processed, we will use the deferred cleanup interface for that
		BeginCleanup(this);
	}
}

void UWorld::InvalidateAllSkyCaptures()
{
	for (TObjectIterator<USkyLightComponent> It; It; ++It)
	{
		USkyLightComponent* CaptureComponent = *It;

		if (ContainsActor(CaptureComponent->GetOwner()) && IsValid(CaptureComponent))
		{
			// Purge cached derived data and force an update
			CaptureComponent->SetCaptureIsDirty();
		}
	}
}

void UWorld::UpdateAllSkyCaptures()
{
	InvalidateAllSkyCaptures();
	USkyLightComponent::UpdateSkyCaptureContents(this);
}

void FSkyLightSceneProxy::Initialize(
	float InBlendFraction, 
	const FSHVectorRGB3* InIrradianceEnvironmentMap, 
	const FSHVectorRGB3* BlendDestinationIrradianceEnvironmentMap,
	const float* InAverageBrightness,
	const float* BlendDestinationAverageBrightness,
	const FLinearColor* InSpecifiedCubemapColorScale)
{
	SpecifiedCubemapColorScale = *InSpecifiedCubemapColorScale;
	BlendFraction = FMath::Clamp(InBlendFraction, 0.0f, 1.0f);
	if (BlendFraction > 0 && BlendDestinationProcessedTexture != NULL)
	{
		if (BlendFraction < 1)
		{
			IrradianceEnvironmentMap = (*InIrradianceEnvironmentMap) * (1 - BlendFraction) + (*BlendDestinationIrradianceEnvironmentMap) * BlendFraction;
			AverageBrightness = *InAverageBrightness * (1 - BlendFraction) + (*BlendDestinationAverageBrightness) * BlendFraction;
		}
		else
		{
			// Blend is full destination, treat as source to avoid blend overhead in shaders
			IrradianceEnvironmentMap = *BlendDestinationIrradianceEnvironmentMap;
			AverageBrightness = *BlendDestinationAverageBrightness;
		}
	}
	else
	{
		// Blend is full source
		IrradianceEnvironmentMap = *InIrradianceEnvironmentMap;
		AverageBrightness = *InAverageBrightness;
		BlendFraction = 0;
	}
}

FLinearColor FSkyLightSceneProxy::GetEffectiveLightColor() const
{
	return LightColor * GSkylightIntensityMultiplier * SpecifiedCubemapColorScale;
}

FSkyLightSceneProxy::FSkyLightSceneProxy(const USkyLightComponent* InLightComponent)
	: LightComponent(InLightComponent)
	, ProcessedTexture(InLightComponent->ProcessedSkyTexture)
	, SkyDistanceThreshold(InLightComponent->SkyDistanceThreshold)
	, BlendDestinationProcessedTexture(InLightComponent->BlendDestinationProcessedSkyTexture)
	, bCastShadows(InLightComponent->CastShadows)
	, bWantsStaticShadowing(InLightComponent->Mobility == EComponentMobility::Stationary)
	, bHasStaticLighting(InLightComponent->HasStaticLighting())
	, bCastVolumetricShadow(InLightComponent->bCastVolumetricShadow)
	, CastRayTracedShadow(InLightComponent->CastRaytracedShadow)
	, bAffectReflection(InLightComponent->bAffectReflection)
	, bAffectGlobalIllumination(InLightComponent->bAffectGlobalIllumination)
	, bTransmission(InLightComponent->bTransmission)
	, OcclusionCombineMode(InLightComponent->OcclusionCombineMode)
	, IndirectLightingIntensity(InLightComponent->IndirectLightingIntensity)
	, VolumetricScatteringIntensity(FMath::Max(InLightComponent->VolumetricScatteringIntensity, 0.0f))
	, OcclusionMaxDistance(InLightComponent->OcclusionMaxDistance)
	, Contrast(InLightComponent->Contrast)
	, OcclusionExponent(FMath::Clamp(InLightComponent->OcclusionExponent, .1f, 10.0f))
	, MinOcclusion(FMath::Clamp(InLightComponent->MinOcclusion, 0.0f, 1.0f))
	, OcclusionTint(InLightComponent->OcclusionTint)
	, bCloudAmbientOcclusion(InLightComponent->bCloudAmbientOcclusion)
	, CloudAmbientOcclusionExtent(InLightComponent->CloudAmbientOcclusionExtent)
	, CloudAmbientOcclusionStrength(InLightComponent->CloudAmbientOcclusionStrength)
	, CloudAmbientOcclusionMapResolutionScale(InLightComponent->CloudAmbientOcclusionMapResolutionScale)
	, CloudAmbientOcclusionApertureScale(InLightComponent->CloudAmbientOcclusionApertureScale)
	, SamplesPerPixel(InLightComponent->SamplesPerPixel)
	, bRealTimeCaptureEnabled(InLightComponent->IsRealTimeCaptureEnabled())
	, CapturePosition(InLightComponent->GetComponentTransform().GetLocation())
	, CaptureCubeMapResolution(InLightComponent->CubemapResolution)
	, LowerHemisphereColor(InLightComponent->LowerHemisphereColor)
	, bLowerHemisphereIsSolidColor(InLightComponent->bLowerHemisphereIsBlack)
#if WITH_EDITOR
	, SecondsToNextIncompleteCapture(0.0f)
	, bCubemapSkyLightWaitingForCubeMapTexture(false)
	, bCaptureSkyLightWaitingForShaders(false)
	, bCaptureSkyLightWaitingForMeshesOrTextures(false)
#endif
	, LightColor(FLinearColor(InLightComponent->LightColor) * InLightComponent->Intensity)
	, bMovable(InLightComponent->IsMovable())
{
	const FSHVectorRGB3* InIrradianceEnvironmentMap = &InLightComponent->IrradianceEnvironmentMap;
	const FSHVectorRGB3* BlendDestinationIrradianceEnvironmentMap = &InLightComponent->BlendDestinationIrradianceEnvironmentMap;
	const float* InAverageBrightness = &InLightComponent->AverageBrightness;
	const float* BlendDestinationAverageBrightness = &InLightComponent->BlendDestinationAverageBrightness;
	float InBlendFraction = InLightComponent->BlendFraction;
	const FLinearColor* InSpecifiedCubemapColorScale = &InLightComponent->SpecifiedCubemapColorScale;
	FSkyLightSceneProxy* LightSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(FInitSkyProxy)(
		[InIrradianceEnvironmentMap, BlendDestinationIrradianceEnvironmentMap, InAverageBrightness, 
		BlendDestinationAverageBrightness, InBlendFraction, LightSceneProxy, InSpecifiedCubemapColorScale] (FRHICommandListBase&)
		{
			// Only access the irradiance maps on the RT, even though they belong to the USkyLightComponent, 
			// Because FScene::UpdateSkyCaptureContents does not block the RT so the writes could still be in flight
			LightSceneProxy->Initialize(InBlendFraction, InIrradianceEnvironmentMap, BlendDestinationIrradianceEnvironmentMap, InAverageBrightness, BlendDestinationAverageBrightness, InSpecifiedCubemapColorScale);
		});
}

USkyLightComponent::USkyLightComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UTexture2D> StaticTexture(TEXT("/Engine/EditorResources/LightIcons/SkyLight"));
		StaticEditorTexture = StaticTexture.Object;
		StaticEditorTextureScale = 1.0f;
		DynamicEditorTexture = StaticTexture.Object;
		DynamicEditorTextureScale = 1.0f;
	}
#endif

	Brightness_DEPRECATED = 1;
	Intensity = 1;
	IndirectLightingIntensity = 1.0f;
	SkyDistanceThreshold = 150000;
	Mobility = EComponentMobility::Stationary;
	bLowerHemisphereIsBlack = true;
	bSavedConstructionScriptValuesValid = true;
	bHasEverCaptured = false;
	OcclusionMaxDistance = 1000;
	MinOcclusion = 0;
	OcclusionExponent = 1;
	OcclusionTint = FColor::Black;
	CubemapResolution = 128;
	LowerHemisphereColor = FLinearColor::Black;
	AverageBrightness = 1.0f;
	BlendDestinationAverageBrightness = 1.0f;
	SpecifiedCubemapColorScale = FLinearColor::White;
	bCastVolumetricShadow = true;
	CastRaytracedShadow = ECastRayTracedShadow::UseProjectSetting;
	bCastRaytracedShadow_DEPRECATED = false;
	bAffectReflection = true;
	bAffectGlobalIllumination = true;
	SamplesPerPixel = 4;
	bRealTimeCapture = false;
	bCloudAmbientOcclusion = 0;
	CloudAmbientOcclusionExtent = 150.0f;
	CloudAmbientOcclusionStrength = 1.0f;
	CloudAmbientOcclusionMapResolutionScale = 1.0f;
	CloudAmbientOcclusionApertureScale = 0.05f;

#if WITH_EDITOR
	CaptureStatus = ESkyLightCaptureStatus::SLCS_Uninitialized;
	SecondsSinceLastCapture = 0.0f;
#endif
}

FSkyLightSceneProxy* USkyLightComponent::CreateSceneProxy() const
{
	if (ProcessedSkyTexture || IsRealTimeCaptureEnabled())
	{
		return new FSkyLightSceneProxy(this);
	}

	return NULL;
}

void USkyLightComponent::SetCaptureIsDirty()
{ 
	if (GetVisibleFlag() && bAffectsWorld && !SkipStaticSkyLightCapture(*this))
	{
		UE::TScopeLock Lock(SkyCapturesToUpdateLock);

#if WITH_EDITOR
		this->CaptureStatus = ESkyLightCaptureStatus::SLCS_Uninitialized;
#endif

		SkyCapturesToUpdate.AddUnique(this);

		// Mark saved values as invalid, in case a sky recapture is requested in a construction script between a save / restore of sky capture state
		bSavedConstructionScriptValuesValid = false;
	}
}

void USkyLightComponent::SanitizeCubemapSize()
{
	const int32 MinCubemapResolution = 8;
	const int32 MaxCubemapResolution = GSkylightCubemapMaxResolution > 0 ? FMath::Clamp(uint32(GSkylightCubemapMaxResolution), uint32(MinCubemapResolution), GetMaxCubeTextureDimension()) : GetMaxCubeTextureDimension();

	CubemapResolution = FMath::Clamp(int32(FMath::RoundUpToPowerOfTwo(CubemapResolution)), MinCubemapResolution, MaxCubemapResolution);

#if WITH_EDITOR
	if (FApp::CanEverRender() && !FApp::IsUnattended())
	{
		SIZE_T TexMemRequired = CalcTextureSize(CubemapResolution, CubemapResolution, SKYLIGHT_CUBEMAP_FORMAT, FMath::CeilLogTwo(CubemapResolution) + 1) * CubeFace_MAX;

		FTextureMemoryStats TextureMemStats;
		RHIGetTextureMemoryStats(TextureMemStats);
		
		int64 DeviceWorkingMemory = TextureMemStats.GetTotalDeviceWorkingMemory();

		if (DeviceWorkingMemory > 0 && TexMemRequired > SIZE_T(DeviceWorkingMemory / 4))
		{
			FNumberFormattingOptions FmtOpts = FNumberFormattingOptions()
				.SetUseGrouping(false)
				.SetMaximumFractionalDigits(2)
				.SetMinimumFractionalDigits(0)
				.SetRoundingMode(HalfFromZero);

			EAppReturnType::Type Response = FPlatformMisc::MessageBoxExt(
				EAppMsgType::YesNo,
				*FText::Format(
					LOCTEXT("MemAllocWarning_Message_SkylightCubemap", "A resolution of {0} will require {1} of video memory. Are you sure?"),
					FText::AsNumber(CubemapResolution, &FmtOpts),
					FText::AsMemory(TexMemRequired, &FmtOpts)
				).ToString(),
				*LOCTEXT("MemAllocWarning_Title_SkylightCubemap", "Memory Allocation Warning").ToString()
			);

			if (Response == EAppReturnType::No)
			{
				CubemapResolution = PreEditCubemapResolution;
			}
		}

		PreEditCubemapResolution = CubemapResolution;
	}
#endif // WITH_EDITOR
}

void USkyLightComponent::SetBlendDestinationCaptureIsDirty()
{ 
	if (GetVisibleFlag() && bAffectsWorld && BlendDestinationCubemap)
	{
		SkyCapturesToUpdateBlendDestinations.AddUnique(this); 

		// Mark saved values as invalid, in case a sky recapture is requested in a construction script between a save / restore of sky capture state
		bSavedConstructionScriptValuesValid = false;
	}
}

TArray<USkyLightComponent*> USkyLightComponent::SkyCapturesToUpdate;
TArray<USkyLightComponent*> USkyLightComponent::SkyCapturesToUpdateBlendDestinations;
FTransactionallySafeCriticalSection USkyLightComponent::SkyCapturesToUpdateLock;

void USkyLightComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	const bool bIsValid = SourceType != SLS_SpecifiedCubemap || Cubemap != NULL || IsRealTimeCaptureEnabled();

	if (bAffectsWorld && bIsValid && ShouldComponentAddToScene() && ShouldRender())
	{
		// Create the light's scene proxy.
		SceneProxy = CreateSceneProxy();

		if (SceneProxy)
		{
			// Add the light to the scene.
			GetWorld()->Scene->SetSkyLight(SceneProxy);
		}
	}
}

void USkyLightComponent::PostInitProperties()
{
	// Skip default object or object belonging to a default object (eg default ASkyLight's component)
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		// Enqueue an update by default, so that newly placed components will get an update
		// PostLoad will undo this for components loaded from disk
		UE::TScopeLock Lock(SkyCapturesToUpdateLock);
		SkyCapturesToUpdate.AddUnique(this);
	}

	Super::PostInitProperties();
}

void USkyLightComponent::PostLoad()
{
	Super::PostLoad();

	SanitizeCubemapSize();

	if (!GIsCookerLoadingPackage)
	{
		// All components are queued for update on creation by default. But we do not want this top happen in some cases.
		if (!GetVisibleFlag() || HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) || SkipStaticSkyLightCapture(*this))
		{
			UE::TScopeLock Lock(SkyCapturesToUpdateLock);
			SkyCapturesToUpdate.Remove(this);
		}
	}
}

/** 
 * Fast path for updating light properties that doesn't require a re-register,
 * Which would otherwise cause the scene's static draw lists to be recreated.
 */
void USkyLightComponent::UpdateLimitedRenderingStateFast()
{
	if (SceneProxy)
	{
		FSkyLightSceneProxy* LightSceneProxy = SceneProxy;
		FLinearColor InLightColor = FLinearColor(LightColor) * Intensity;
		float InIndirectLightingIntensity = IndirectLightingIntensity;
		float InVolumetricScatteringIntensity = VolumetricScatteringIntensity;
		FLinearColor InLowerHemisphereColor = LowerHemisphereColor;
		ENQUEUE_RENDER_COMMAND(FFastUpdateSkyLightCommand)(
			[LightSceneProxy, InLightColor, InIndirectLightingIntensity, InVolumetricScatteringIntensity, InLowerHemisphereColor] (FRHICommandListBase&)
			{
				LightSceneProxy->SetLightColor(InLightColor);
				LightSceneProxy->IndirectLightingIntensity = InIndirectLightingIntensity;
				LightSceneProxy->VolumetricScatteringIntensity = InVolumetricScatteringIntensity;
				LightSceneProxy->LowerHemisphereColor = InLowerHemisphereColor;
			});
	}
}

void USkyLightComponent::UpdateOcclusionRenderingStateFast()
{
	if (SceneProxy && IsOcclusionSupported())
	{
		FSkyLightSceneProxy* InLightSceneProxy = SceneProxy;
		float InContrast = Contrast;
		float InOcclusionExponent = OcclusionExponent;
		float InMinOcclusion = MinOcclusion;
		FColor InOcclusionTint = OcclusionTint;
		ENQUEUE_RENDER_COMMAND(FFastUpdateSkyLightOcclusionCommand)(
			[InLightSceneProxy, InContrast, InOcclusionExponent, InMinOcclusion, InOcclusionTint] (FRHICommandListBase&)
			{
				InLightSceneProxy->Contrast = InContrast;
				InLightSceneProxy->OcclusionExponent = InOcclusionExponent;
				InLightSceneProxy->MinOcclusion = InMinOcclusion;
				InLightSceneProxy->OcclusionTint = InOcclusionTint;
			});
	}

}

void USkyLightComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	if (SceneProxy)
	{
		GetWorld()->Scene->DisableSkyLight(SceneProxy);

		FSkyLightSceneProxy* LightSceneProxy = SceneProxy;
		ENQUEUE_RENDER_COMMAND(FDestroySkyLightCommand)(
			[LightSceneProxy] (FRHICommandListBase&)
			{
				delete LightSceneProxy;
			});

		SceneProxy = nullptr;
	}
}

void USkyLightComponent::SendRenderTransform_Concurrent()
{
	if (SceneProxy)
	{
		FSkyLightSceneProxy* InLightSceneProxy = SceneProxy;
		FVector Position = GetComponentTransform().GetLocation();

		ENQUEUE_RENDER_COMMAND(UpdateSkyLightCapturePosition)(
			[InLightSceneProxy, Position] (FRHICommandListBase&)
			{
				InLightSceneProxy->CapturePosition = Position;
			});
	}

	Super::SendRenderTransform_Concurrent();
}

#if WITH_EDITOR
void USkyLightComponent::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);
	PreEditCubemapResolution = CubemapResolution;
}

void USkyLightComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USkyLightComponent, CubemapResolution))
	{
		// Simply rounds the cube map size to nearest power of two. Occasionally checks for out of video mem.
		SanitizeCubemapSize();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	SetCaptureIsDirty();
}

bool USkyLightComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (FCString::Strcmp(*PropertyName, TEXT("Cubemap")) == 0
			|| FCString::Strcmp(*PropertyName, TEXT("SourceCubemapAngle")) == 0)
		{
			return SourceType == SLS_SpecifiedCubemap;
		}

		if (FCString::Strcmp(*PropertyName, TEXT("LowerHemisphereColor")) == 0)
		{
			return bLowerHemisphereIsBlack;
		}

		if (FCString::Strcmp(*PropertyName, TEXT("bRealTimeCapture")) == 0)
		{
			return Mobility == EComponentMobility::Movable || Mobility == EComponentMobility::Stationary;
		}
		if (FCString::Strcmp(*PropertyName, TEXT("SourceType")) == 0)
		{
			return !IsRealTimeCaptureEnabled();
		}

		if (FCString::Strcmp(*PropertyName, TEXT("CloudAmbientOcclusionExtent")) == 0 
			|| FCString::Strcmp(*PropertyName, TEXT("CloudAmbientOcclusionMapResolutionScale")) == 0
			|| FCString::Strcmp(*PropertyName, TEXT("CloudAmbientOcclusionStrength")) == 0
			|| FCString::Strcmp(*PropertyName, TEXT("CloudAmbientOcclusionApertureScale")) == 0)
		{
			return bCloudAmbientOcclusion;
		}

		if (FCString::Strcmp(*PropertyName, TEXT("Contrast")) == 0
			|| FCString::Strcmp(*PropertyName, TEXT("OcclusionMaxDistance")) == 0
			|| FCString::Strcmp(*PropertyName, TEXT("MinOcclusion")) == 0
			|| FCString::Strcmp(*PropertyName, TEXT("OcclusionTint")) == 0)
		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));
			return Mobility == EComponentMobility::Movable && CastShadows && CVar->GetValueOnGameThread() != 0;
		}

		if (FCString::Strcmp(*PropertyName, TEXT("CastRaytracedShadow")) == 0)
		{
			return IsRayTracingEnabled();
		}
	}

	return Super::CanEditChange(InProperty);
}

void USkyLightComponent::CheckForErrors()
{
	AActor* Owner = GetOwner();

	if (Owner && GetVisibleFlag() && bAffectsWorld)
	{
		UWorld* ThisWorld = Owner->GetWorld();
		bool bMultipleFound = false;

		if (ThisWorld)
		{
			for (TObjectIterator<USkyLightComponent> ComponentIt; ComponentIt; ++ComponentIt)
			{
				USkyLightComponent* Component = *ComponentIt;

				if (Component != this 
					&& IsValid(Component)
					&& Component->GetVisibleFlag()
					&& Component->bAffectsWorld
					&& Component->GetOwner() 
					&& ThisWorld->ContainsActor(Component->GetOwner())
					&& IsValid(Component->GetOwner()))
				{
					bMultipleFound = true;
					break;
				}
			}
		}

		if (bMultipleFound)
		{
			FMessageLog("MapCheck").Error()
				->AddToken(FUObjectToken::Create(Owner))
				->AddToken(FTextToken::Create(LOCTEXT( "MapCheck_Message_MultipleSkyLights", "Multiple sky lights are active, only one can be enabled per world." )))
				->AddToken(FMapErrorToken::Create(FMapErrors::MultipleSkyLights));
		}
	}
}

#endif // WITH_EDITOR

void USkyLightComponent::BeginDestroy()
{
	// Deregister the component from the update queue
	{
		UE::TScopeLock Lock(SkyCapturesToUpdateLock); 
		SkyCapturesToUpdate.Remove(this);
	}
	
	SkyCapturesToUpdateBlendDestinations.Remove(this);

	// Release reference
	ProcessedSkyTexture = NULL;

	// Begin a fence to track the progress of the above BeginReleaseResource being completed on the RT
	ReleaseResourcesFence.BeginFence();

	Super::BeginDestroy();
}

bool USkyLightComponent::IsReadyForFinishDestroy()
{
	// Wait until the fence is complete before allowing destruction
	return Super::IsReadyForFinishDestroy() && ReleaseResourcesFence.IsFenceComplete();
}

TStructOnScope<FActorComponentInstanceData> USkyLightComponent::GetComponentInstanceData() const
{
	TStructOnScope<FActorComponentInstanceData> InstanceData = MakeStructOnScope<FActorComponentInstanceData, FPrecomputedSkyLightInstanceData>(this);
	FPrecomputedSkyLightInstanceData* SkyLightInstanceData = InstanceData.Cast<FPrecomputedSkyLightInstanceData>();
	SkyLightInstanceData->OriginalLightGuid = OriginalLightGuid;
	SkyLightInstanceData->LightGuid = LightGuid;
	SkyLightInstanceData->ProcessedSkyTexture = ProcessedSkyTexture;

	// Block until the rendering thread has completed its writes from a previous capture
	IrradianceMapFence.Wait();
	SkyLightInstanceData->IrradianceEnvironmentMap = IrradianceEnvironmentMap;
	SkyLightInstanceData->AverageBrightness = AverageBrightness;
	// RHI_RAYTRACING #SkyLightIS @todo:
	return InstanceData;
}

void USkyLightComponent::ApplyComponentInstanceData(FPrecomputedSkyLightInstanceData* LightMapData)
{
	check(LightMapData);

	OriginalLightGuid = (HasStaticShadowing() ? LightMapData->OriginalLightGuid : FGuid());
	LightGuid = (HasStaticShadowing() ? LightMapData->LightGuid : FGuid());
	ProcessedSkyTexture = LightMapData->ProcessedSkyTexture;
	IrradianceEnvironmentMap = LightMapData->IrradianceEnvironmentMap;
	AverageBrightness = LightMapData->AverageBrightness;

	if (ProcessedSkyTexture && bSavedConstructionScriptValuesValid)
	{
		// We have valid capture state, remove the queued update
		UE::TScopeLock Lock(SkyCapturesToUpdateLock);
		SkyCapturesToUpdate.Remove(this);
	}

	MarkRenderStateDirty();
}

void USkyLightComponent::UpdateSkyCaptureContentsArray(UWorld* WorldToUpdate, TArray<USkyLightComponent*>& ComponentArray, bool bOperateOnBlendSource)
{
	const bool bIsCompilingShaders = GShaderCompilingManager != nullptr && GShaderCompilingManager->IsCompiling();
	bool bSceneIsAsyncCompiling = false;
#if WITH_EDITOR
	bSceneIsAsyncCompiling = FTextureCompilingManager::Get().GetNumRemainingTextures() > 0 || FStaticMeshCompilingManager::Get().GetNumRemainingMeshes() > 0;
#endif

	// Iterate backwards so we can remove elements without changing the index
	for (int32 CaptureIndex = ComponentArray.Num() - 1; CaptureIndex >= 0; CaptureIndex--)
	{
		USkyLightComponent* CaptureComponent = ComponentArray[CaptureIndex];
		AActor* Owner = CaptureComponent->GetOwner();

		if (CaptureComponent->GetWorld() != WorldToUpdate)
		{
			continue;
		}

		// Reset the luminance scale in case the texture has been switched
		CaptureComponent->SpecifiedCubemapColorScale = FLinearColor::White;

		// For specific cubemaps, we must wait until the texture is compiled before capturing the skylight
		bool bIsCubemapCompiling = false;
#if WITH_EDITOR
		bIsCubemapCompiling =
			CaptureComponent->SourceType == SLS_SpecifiedCubemap &&
			CaptureComponent->Cubemap &&
			CaptureComponent->Cubemap->IsDefaultTexture();

		if (bIsCubemapCompiling)
		{
			// We should process this texture as soon as possible so we can have a proper skylight.
			FTextureCompilingManager::Get().RequestPriorityChange(CaptureComponent->Cubemap, EQueuedWorkPriority::Highest);
		}

		const float SecondsBetweenIncompleteCaptures = 5.0f;
		const bool bCubemapSkyLightWaitingForCubemapAsset	= CaptureComponent->SourceType == SLS_SpecifiedCubemap	&& bIsCubemapCompiling;
		const bool bCaptureSkyLightWaitingCompiledShader	= CaptureComponent->SourceType == SLS_CapturedScene		&& bIsCompilingShaders;
		const bool bCaptureSkyLightWaitingForMeshOrTexAssets= CaptureComponent->SourceType == SLS_CapturedScene		&& bSceneIsAsyncCompiling;
#endif

		if ((!Owner || !Owner->GetLevel() || Owner->GetLevel()->bIsVisible)
			// Only process sky capture requests once async texture and shader compiling completes, otherwise we will capture the scene with temporary shaders/textures
			&& (
#if WITH_EDITOR
				CaptureComponent->CaptureStatus == ESkyLightCaptureStatus::SLCS_Uninitialized
				|| 
				(CaptureComponent->CaptureStatus == ESkyLightCaptureStatus::SLCS_CapturedButIncomplete && CaptureComponent->SecondsSinceLastCapture > SecondsBetweenIncompleteCaptures)
				||
#endif
				((!bSceneIsAsyncCompiling) && (!bIsCompilingShaders)) 
				|| 
				((CaptureComponent->SourceType == SLS_SpecifiedCubemap) && (!bIsCubemapCompiling)))
			)
		{
			// Only capture valid sky light components
			if (CaptureComponent->SourceType != SLS_SpecifiedCubemap || CaptureComponent->Cubemap)
			{
#if WITH_EDITOR
				FStaticLightingSystemInterface::OnLightComponentUnregistered.Broadcast(CaptureComponent);
#endif

				if (bOperateOnBlendSource)
				{
					ensure(!CaptureComponent->ProcessedSkyTexture || CaptureComponent->ProcessedSkyTexture->GetSizeX() == CaptureComponent->ProcessedSkyTexture->GetSizeY());

					// Allocate the needed texture on first capture
					if (!CaptureComponent->ProcessedSkyTexture || CaptureComponent->ProcessedSkyTexture->GetSizeX() != CaptureComponent->CubemapResolution)
					{
						CaptureComponent->ProcessedSkyTexture = new FSkyTextureCubeResource();
						CaptureComponent->ProcessedSkyTexture->SetupParameters(CaptureComponent->CubemapResolution, FMath::CeilLogTwo(CaptureComponent->CubemapResolution) + 1, SKYLIGHT_CUBEMAP_FORMAT);
						BeginInitResource(CaptureComponent->ProcessedSkyTexture);
						CaptureComponent->MarkRenderStateDirty();
					}

					WorldToUpdate->Scene->UpdateSkyCaptureContents(CaptureComponent, CaptureComponent->bCaptureEmissiveOnly, CaptureComponent->Cubemap, CaptureComponent->ProcessedSkyTexture, CaptureComponent->AverageBrightness, CaptureComponent->IrradianceEnvironmentMap, NULL, &CaptureComponent->SpecifiedCubemapColorScale);
				}
				else
				{
					ensure(!CaptureComponent->BlendDestinationProcessedSkyTexture || CaptureComponent->BlendDestinationProcessedSkyTexture->GetSizeX() == CaptureComponent->BlendDestinationProcessedSkyTexture->GetSizeY());

					// Allocate the needed texture on first capture
					if (!CaptureComponent->BlendDestinationProcessedSkyTexture || CaptureComponent->BlendDestinationProcessedSkyTexture->GetSizeX() != CaptureComponent->CubemapResolution)
					{
						CaptureComponent->BlendDestinationProcessedSkyTexture = new FSkyTextureCubeResource();
						CaptureComponent->BlendDestinationProcessedSkyTexture->SetupParameters(CaptureComponent->CubemapResolution, FMath::CeilLogTwo(CaptureComponent->CubemapResolution) + 1, SKYLIGHT_CUBEMAP_FORMAT);
						BeginInitResource(CaptureComponent->BlendDestinationProcessedSkyTexture);
						CaptureComponent->MarkRenderStateDirty(); 
					}

					WorldToUpdate->Scene->UpdateSkyCaptureContents(CaptureComponent, CaptureComponent->bCaptureEmissiveOnly, CaptureComponent->BlendDestinationCubemap, CaptureComponent->BlendDestinationProcessedSkyTexture, CaptureComponent->BlendDestinationAverageBrightness, CaptureComponent->BlendDestinationIrradianceEnvironmentMap, NULL, &CaptureComponent->SpecifiedCubemapColorScale);
				}

				CaptureComponent->IrradianceMapFence.BeginFence();
				CaptureComponent->bHasEverCaptured = true;
				CaptureComponent->MarkRenderStateDirty();

#if WITH_EDITOR
				FStaticLightingSystemInterface::OnLightComponentRegistered.Broadcast(CaptureComponent);
#endif
			}

#if WITH_EDITOR
			const bool bCaptureIsComplete = !bCubemapSkyLightWaitingForCubemapAsset && !bCaptureSkyLightWaitingCompiledShader && !bCaptureSkyLightWaitingForMeshOrTexAssets;
			switch (CaptureComponent->CaptureStatus)
			{
			case ESkyLightCaptureStatus::SLCS_Uninitialized:
			{
				CaptureComponent->SecondsSinceLastCapture = 0.0f;

				if (bCaptureIsComplete)
				{
					// Do not recapture if the first forced capture was with a complete world (to avoid capturing twice each time a level is loaded or a skylight created))
					CaptureComponent->CaptureStatus = ESkyLightCaptureStatus::SLCS_CapturedAndComplete;
					ComponentArray.RemoveAt(CaptureIndex);
				}
				else
				{
					CaptureComponent->CaptureStatus = ESkyLightCaptureStatus::SLCS_CapturedButIncomplete;
				}
				break;
			}
			case ESkyLightCaptureStatus::SLCS_CapturedButIncomplete:
			{
				if (bCaptureIsComplete)
				{
					// Only remove queued update requests if we processed it for the a world with all meshes, textures and shaders.
					ComponentArray.RemoveAt(CaptureIndex);
					CaptureComponent->CaptureStatus = ESkyLightCaptureStatus::SLCS_CapturedAndComplete;
				}
				else if (CaptureComponent->SecondsSinceLastCapture > SecondsBetweenIncompleteCaptures)
				{
					// We have just executed another incomplete capture, so reset the timer for the next one.
					CaptureComponent->SecondsSinceLastCapture = 0.0f;
				}
				break;
			}
			case ESkyLightCaptureStatus::SLCS_CapturedAndComplete:
			{
				// It is valid to recapture a complete skylight.
				ComponentArray.RemoveAt(CaptureIndex);
				CaptureComponent->SecondsSinceLastCapture = 0.0f;
				break;
			}
			default:
			{
				check(false);
				break;
			}
			}
#else
			ComponentArray.RemoveAt(CaptureIndex);
#endif
		}

#if WITH_EDITOR
		CaptureComponent->SecondsSinceLastCapture += CaptureComponent->CaptureStatus == ESkyLightCaptureStatus::SLCS_CapturedButIncomplete ? WorldToUpdate->DeltaTimeSeconds : 0.0f;

		ENQUEUE_RENDER_COMMAND(FUpdateSkyLightProxyStatusForcedCapture)(
			[CaptureComponent, SecondsBetweenIncompleteCaptures, bCubemapSkyLightWaitingForCubemapAsset, bCaptureSkyLightWaitingCompiledShader, bCaptureSkyLightWaitingForMeshOrTexAssets] (FRHICommandListBase&)
			{
				FSkyLightSceneProxy* SkyLightSceneProxy = CaptureComponent->SceneProxy;
				if (SkyLightSceneProxy)
				{
					SkyLightSceneProxy->SecondsToNextIncompleteCapture = FMath::Max(0.0f, SecondsBetweenIncompleteCaptures - CaptureComponent->SecondsSinceLastCapture);
					SkyLightSceneProxy->bCubemapSkyLightWaitingForCubeMapTexture = bCubemapSkyLightWaitingForCubemapAsset;
					SkyLightSceneProxy->bCaptureSkyLightWaitingForShaders = bCaptureSkyLightWaitingCompiledShader;
					SkyLightSceneProxy->bCaptureSkyLightWaitingForMeshesOrTextures = bCaptureSkyLightWaitingForMeshOrTexAssets;
				}
			});
#endif
	}
}

bool USkyLightComponent::HasSkyCapturesToUpdate()
{
	if (GUpdateSkylightsEveryFrame || !SkyCapturesToUpdateBlendDestinations.IsEmpty())
	{
		return true;
	}
	UE::TScopeLock Lock(SkyCapturesToUpdateLock);
	return !SkyCapturesToUpdate.IsEmpty();
}

void USkyLightComponent::UpdateSkyCaptureContents(UWorld* WorldToUpdate)
{
	if (WorldToUpdate && WorldToUpdate->Scene)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_SkylightCaptures);

		if (GUpdateSkylightsEveryFrame)
		{
			for (TObjectIterator<USkyLightComponent> It; It; ++It)
			{
				USkyLightComponent* SkylightComponent = *It;
				if (WorldToUpdate->ContainsActor(SkylightComponent->GetOwner()) && IsValid(SkylightComponent))
				{			
					SkylightComponent->SetCaptureIsDirty();			
				}
			}
		}

		{
			UE::TScopeLock Lock(SkyCapturesToUpdateLock);
			if (SkyCapturesToUpdate.Num() > 0)
			{
				// Remove the sky captures if real time capture is enabled. 
				SkyCapturesToUpdate.RemoveAll([WorldToUpdate](const USkyLightComponent* CaptureComponent) { return CaptureComponent->GetWorld() == WorldToUpdate && CaptureComponent->IsRealTimeCaptureEnabled(); });
				UpdateSkyCaptureContentsArray(WorldToUpdate, SkyCapturesToUpdate, true);
			}
		}
		
		if (SkyCapturesToUpdateBlendDestinations.Num() > 0)
		{
			UpdateSkyCaptureContentsArray(WorldToUpdate, SkyCapturesToUpdateBlendDestinations, false);
		}
	}
}

void USkyLightComponent::CaptureEmissiveRadianceEnvironmentCubeMap(FSHVectorRGB3& OutIrradianceMap, TArray<FFloat16Color>& OutRadianceMap) const
{
	OutIrradianceMap = FSHVectorRGB3();
	if (GetScene() && (SourceType != SLS_SpecifiedCubemap || Cubemap))
	{
		float UnusedAverageBrightness = 1.0f;
		FLinearColor* UnusedSpecifiedCubemapColorScale = nullptr;	// Disable
		// Capture emissive scene lighting only for the lighting build
		// This is necessary to avoid a feedback loop with the last lighting build results
		GetScene()->UpdateSkyCaptureContents(this, true, Cubemap, NULL, UnusedAverageBrightness, OutIrradianceMap, &OutRadianceMap, UnusedSpecifiedCubemapColorScale);
		// Wait until writes to OutIrradianceMap have completed
		FlushRenderingCommands();
	}
}

/** Set brightness of the light */
void USkyLightComponent::SetIntensity(float NewIntensity)
{
	// Can't set brightness on a static light
	if (AreDynamicDataChangesAllowed()
		&& Intensity != NewIntensity)
	{
		Intensity = NewIntensity;
		UpdateLimitedRenderingStateFast();
	}
}

void USkyLightComponent::SetIndirectLightingIntensity(float NewIntensity)
{
	// Can't set brightness on a static light
	if (AreDynamicDataChangesAllowed()
		&& IndirectLightingIntensity != NewIntensity)
	{
		IndirectLightingIntensity = NewIntensity;
		UpdateLimitedRenderingStateFast();
	}
}

void USkyLightComponent::SetVolumetricScatteringIntensity(float NewIntensity)
{
	// Can't set brightness on a static light
	if (AreDynamicDataChangesAllowed()
		&& VolumetricScatteringIntensity != NewIntensity)
	{
		VolumetricScatteringIntensity = NewIntensity;
		UpdateLimitedRenderingStateFast();
	}
}

/** Set color of the light */
void USkyLightComponent::SetLightColor(FLinearColor NewLightColor)
{
	FColor NewColor(NewLightColor.ToFColor(true));

	// Can't set color on a static light
	if (AreDynamicDataChangesAllowed()
		&& LightColor != NewColor)
	{
		LightColor = NewColor;
		UpdateLimitedRenderingStateFast();
	}
}

void USkyLightComponent::SetCubemap(UTextureCube* NewCubemap)
{
	// Can't set on a static light
	if (AreDynamicDataChangesAllowed()
		&& Cubemap != NewCubemap)
	{
		Cubemap = NewCubemap;
		MarkRenderStateDirty();
		// Note: this will cause the cubemap to be reprocessed including readback from the GPU
		SetCaptureIsDirty();
	}
}

void USkyLightComponent::SetSourceCubemapAngle(float NewValue)
{
	// Can't set on a static light
	if (AreDynamicDataChangesAllowed()
		&& SourceCubemapAngle != NewValue)
	{
		SourceCubemapAngle = NewValue;
		MarkRenderStateDirty();
		// Note: this will cause the cubemap to be reprocessed including readback from the GPU
		SetCaptureIsDirty();
	}
}

void USkyLightComponent::SetCubemapBlend(UTextureCube* SourceCubemap, UTextureCube* DestinationCubemap, float InBlendFraction)
{
	if (AreDynamicDataChangesAllowed()
		&& (Cubemap != SourceCubemap || BlendDestinationCubemap != DestinationCubemap || BlendFraction != InBlendFraction)
		&& SourceType == SLS_SpecifiedCubemap)
	{
		if (Cubemap != SourceCubemap)
		{
			Cubemap = SourceCubemap;
			SetCaptureIsDirty();
		}

		if (BlendDestinationCubemap != DestinationCubemap)
		{
			BlendDestinationCubemap = DestinationCubemap;
			SetBlendDestinationCaptureIsDirty();
		}

		if (BlendFraction != InBlendFraction)
		{
			BlendFraction = InBlendFraction;

			if (SceneProxy)
			{
				const FSHVectorRGB3* InIrradianceEnvironmentMap = &IrradianceEnvironmentMap;
				const FSHVectorRGB3* InBlendDestinationIrradianceEnvironmentMap = &BlendDestinationIrradianceEnvironmentMap;
				const float* InAverageBrightness = &AverageBrightness;
				const float* InBlendDestinationAverageBrightness = &BlendDestinationAverageBrightness;
				FSkyLightSceneProxy* LightSceneProxy = SceneProxy;
				const FLinearColor* InSpecifiedCubemapColorScale = &SpecifiedCubemapColorScale;
				ENQUEUE_RENDER_COMMAND(FUpdateSkyProxy)(
					[InIrradianceEnvironmentMap, InBlendDestinationIrradianceEnvironmentMap, InAverageBrightness, InBlendDestinationAverageBrightness, InBlendFraction, LightSceneProxy, InSpecifiedCubemapColorScale] (FRHICommandListBase&)
					{
						// Only access the irradiance maps on the RT, even though they belong to the USkyLightComponent, 
						// Because FScene::UpdateSkyCaptureContents does not block the RT so the writes could still be in flight
						LightSceneProxy->Initialize(InBlendFraction, InIrradianceEnvironmentMap, InBlendDestinationIrradianceEnvironmentMap, InAverageBrightness, InBlendDestinationAverageBrightness, InSpecifiedCubemapColorScale);
					});
			}
		}
	}
}

void USkyLightComponent::SetLowerHemisphereColor(const FLinearColor& InLowerHemisphereColor)
{
	// Can't set on a static light
	if (AreDynamicDataChangesAllowed()
		&& LowerHemisphereColor != InLowerHemisphereColor)
	{
		LowerHemisphereColor = InLowerHemisphereColor;
		UpdateLimitedRenderingStateFast();
	}
}

void USkyLightComponent::SetOcclusionTint(const FColor& InTint)
{
	// Can't set on a static light
	if (AreDynamicDataChangesAllowed()
		&& OcclusionTint != InTint)
	{
		OcclusionTint = InTint;
		UpdateOcclusionRenderingStateFast();
	}
}

void USkyLightComponent::SetOcclusionContrast(float InOcclusionContrast)
{
	if (AreDynamicDataChangesAllowed()
		&& Contrast != InOcclusionContrast)
	{
		Contrast = InOcclusionContrast;
		UpdateOcclusionRenderingStateFast();
	}
}

void USkyLightComponent::SetOcclusionExponent(float InOcclusionExponent)
{
	if (AreDynamicDataChangesAllowed()
		&& OcclusionExponent != InOcclusionExponent)
	{
		OcclusionExponent = InOcclusionExponent;
		UpdateOcclusionRenderingStateFast();
	}
}

void USkyLightComponent::SetMinOcclusion(float InMinOcclusion)
{
	// Can't set on a static light
	if (AreDynamicDataChangesAllowed()
		&& MinOcclusion != InMinOcclusion)
	{
		MinOcclusion = InMinOcclusion;
		UpdateOcclusionRenderingStateFast();
	}
}

void USkyLightComponent::SetRealTimeCapture(bool bInRealTimeCapture)
{
	if (AreDynamicDataChangesAllowed()
		&& bInRealTimeCapture != bRealTimeCapture)
	{
		SetRealTimeCaptureEnabled(bInRealTimeCapture);
	}
}

bool USkyLightComponent::IsOcclusionSupported() const
{
	FSceneInterface* LocalScene = GetScene();
	if (LocalScene && LocalScene->GetFeatureLevel() <= ERHIFeatureLevel::ES3_1)
	{
		// Sky occlusion is not supported on mobile
		return false;
	}
	return true;
}

bool USkyLightComponent::IsRealTimeCaptureEnabled() const
{
	return bRealTimeCapture && (Mobility == EComponentMobility::Movable || Mobility == EComponentMobility::Stationary) && GSkylightRealTimeReflectionCapture > 0;
}

void USkyLightComponent::SetRealTimeCaptureEnabled(bool bNewRealTimeCaptureEnabled)
{
	bRealTimeCapture = bNewRealTimeCaptureEnabled;
	MarkRenderStateDirty();
	SetCaptureIsDirty();
}

void USkyLightComponent::OnVisibilityChanged()
{
	Super::OnVisibilityChanged();

	if (GetVisibleFlag() && !bHasEverCaptured)
	{
		// Capture if we are being enabled for the first time
		SetCaptureIsDirty();
		SetBlendDestinationCaptureIsDirty();
	}
}

void USkyLightComponent::RecaptureSky()
{
	SetCaptureIsDirty();
}

void USkyLightComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Super::Serialize(Ar);

	// if version is between VER_UE4_SKYLIGHT_MOBILE_IRRADIANCE_MAP and FReleaseObjectVersion::SkyLightRemoveMobileIrradianceMap then handle aborted attempt to serialize irradiance data on mobile.
	if (Ar.UEVer() >= VER_UE4_SKYLIGHT_MOBILE_IRRADIANCE_MAP && !(Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::SkyLightRemoveMobileIrradianceMap))
	{
		FSHVectorRGB3 DummyIrradianceEnvironmentMap;
		Ar << DummyIrradianceEnvironmentMap;
	}

	if (Ar.IsLoading() && (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RayTracedShadowsType))
	{
		CastRaytracedShadow = bCastRaytracedShadow_DEPRECATED == 0 ? ECastRayTracedShadow::Disabled : ECastRayTracedShadow::Enabled;
	}
}



ASkyLight::ASkyLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LightComponent = CreateDefaultSubobject<USkyLightComponent>(TEXT("SkyLightComponent0"));
	RootComponent = LightComponent;
	SetHidden(false);
#if WITH_EDITORONLY_DATA
	// Null out the sprite. The Skylight components sprite is the one we use.
	if (GetSpriteComponent())
	{
		GetSpriteComponent()->Sprite = nullptr;
	}
#endif // WITH_EDITORONLY_DATA
}

void ASkyLight::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );

	DOREPLIFETIME( ASkyLight, bEnabled );
}

void ASkyLight::OnRep_bEnabled()
{
	LightComponent->SetVisibility(bEnabled);
}


#undef LOCTEXT_NAMESPACE
