// Copyright Epic Games, Inc. All Rights Reserved.


#include "Components/SphereComponent.h"
#include "CollisionShape.h"
#include "PhysicsEngine/BodySetup.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "PhysicsEngine/SphereElem.h"
#include "SceneManagement.h"
#include "PrimitiveSceneProxy.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SphereComponent)

USphereComponent::USphereComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SphereRadius = 32.0f;
	ShapeColor = FColor(255, 0, 0, 255);

	bUseEditorCompositing = true;
}

FBoxSphereBounds USphereComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds( FVector::ZeroVector, FVector(SphereRadius), SphereRadius ).TransformBy(LocalToWorld);
}

void USphereComponent::CalcBoundingCylinder(float& CylinderRadius, float& CylinderHalfHeight) const
{
	CylinderRadius = SphereRadius * GetComponentTransform().GetMaximumAxisScale();
	CylinderHalfHeight = CylinderRadius;
}

template <EShapeBodySetupHelper UpdateBodySetupAction, typename BodySetupType>
bool InvalidateOrUpdateSphereBodySetup(BodySetupType& ShapeBodySetup, bool bUseArchetypeBodySetup, float SphereRadius)
{
	check((bUseArchetypeBodySetup && UpdateBodySetupAction == EShapeBodySetupHelper::InvalidateSharingIfStale) || (!bUseArchetypeBodySetup && UpdateBodySetupAction == EShapeBodySetupHelper::UpdateBodySetup) );
	check(ShapeBodySetup->AggGeom.SphereElems.Num() == 1);
	FKSphereElem* SphereElem = ShapeBodySetup->AggGeom.SphereElems.GetData();

	// check for mal formed values
	float Radius = SphereRadius;
	if (Radius < UE_KINDA_SMALL_NUMBER)
	{
		Radius = 0.1f;
	}

	if(UpdateBodySetupAction == EShapeBodySetupHelper::UpdateBodySetup)
	{
		// now set the PhysX data values
		SphereElem->Center = FVector::ZeroVector;
		SphereElem->Radius = Radius;
	}
	else
	{
		if(SphereElem->Radius != Radius)
		{
			ShapeBodySetup = nullptr;
			bUseArchetypeBodySetup = false;
		}
	}
	
	return bUseArchetypeBodySetup;
}

void USphereComponent::UpdateBodySetup()
{
	if(PrepareSharedBodySetup<USphereComponent>())
	{
		bUseArchetypeBodySetup = InvalidateOrUpdateSphereBodySetup<EShapeBodySetupHelper::InvalidateSharingIfStale>(ShapeBodySetup, bUseArchetypeBodySetup, SphereRadius);
	}

	CreateShapeBodySetupIfNeeded<FKSphereElem>();

	if(!bUseArchetypeBodySetup)
	{
		InvalidateOrUpdateSphereBodySetup<EShapeBodySetupHelper::UpdateBodySetup>(ShapeBodySetup, bUseArchetypeBodySetup, SphereRadius);
	}
}

void USphereComponent::SetSphereRadius( float InSphereRadius, bool bUpdateOverlaps )
{
	SphereRadius = InSphereRadius;
	UpdateBounds();
	UpdateBodySetup();
	MarkRenderStateDirty();

	if (bPhysicsStateCreated)
	{
		// Update physics engine collision shapes
		BodyInstance.UpdateBodyScale(GetComponentTransform().GetScale3D(), true);

		if ( bUpdateOverlaps && IsCollisionEnabled() && GetOwner() )
		{
			UpdateOverlaps();
		}
	}
}

bool USphereComponent::IsZeroExtent() const
{
	return SphereRadius == 0.f;
}


FPrimitiveSceneProxy* USphereComponent::CreateSceneProxy()
{
	/** Represents a DrawLightRadiusComponent to the scene manager. */
	class FSphereSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		/** Initialization constructor. */
		FSphereSceneProxy(const USphereComponent* InComponent)
			:	FPrimitiveSceneProxy(InComponent)
			,	bDrawOnlyIfSelected( InComponent->bDrawOnlyIfSelected )
			,	SphereColor(InComponent->ShapeColor)
			,	SphereRadius(InComponent->SphereRadius)
			,	LineThickness(InComponent->LineThickness)
		{
			bWillEverBeLit = false;
		}

		  // FPrimitiveSceneProxy interface.
		
		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER( STAT_SphereSceneProxy_GetDynamicMeshElements );

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

					const FMatrix& LocalToWorld = GetLocalToWorld();
					const FLinearColor DrawSphereColor = GetViewSelectionColor(SphereColor, *View, IsSelected(), IsHovered(), false, IsIndividuallySelected() );

					// Taking into account the min and maximum drawing distance
					const float DistanceSqr = (View->ViewMatrices.GetViewOrigin() - LocalToWorld.GetOrigin()).SizeSquared();
					if (DistanceSqr < FMath::Square(GetMinDrawDistance()) || DistanceSqr > FMath::Square(GetMaxDrawDistance()) )
					{
						continue;
					}
					
					float AbsScaleX = LocalToWorld.GetScaledAxis(EAxis::X).Size();
					float AbsScaleY = LocalToWorld.GetScaledAxis(EAxis::Y).Size();
					float AbsScaleZ = LocalToWorld.GetScaledAxis(EAxis::Z).Size();
					float MinAbsScale = FMath::Min3(AbsScaleX, AbsScaleY, AbsScaleZ);

					FVector ScaledX = LocalToWorld.GetUnitAxis(EAxis::X) * MinAbsScale;
					FVector ScaledY = LocalToWorld.GetUnitAxis(EAxis::Y) * MinAbsScale;
					FVector ScaledZ = LocalToWorld.GetUnitAxis(EAxis::Z) * MinAbsScale;

					const int32 SphereSides = FMath::Clamp<int32>(SphereRadius / 4.f, 16, 64);
					DrawCircle(PDI, LocalToWorld.GetOrigin(), ScaledX, ScaledY, DrawSphereColor, SphereRadius, SphereSides, SDPG_World, LineThickness);
					DrawCircle(PDI, LocalToWorld.GetOrigin(), ScaledX, ScaledZ, DrawSphereColor, SphereRadius, SphereSides, SDPG_World, LineThickness);
					DrawCircle(PDI, LocalToWorld.GetOrigin(), ScaledY, ScaledZ, DrawSphereColor, SphereRadius, SphereSides, SDPG_World, LineThickness);
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			const bool bVisibleForSelection = !bDrawOnlyIfSelected || IsSelected();
			const bool bVisibleForShowFlags = true; // @TODO

			// Should we draw this because collision drawing is enabled, and we have collision
			const bool bShowForCollision = View->Family->EngineShowFlags.Collision && IsCollisionEnabled();

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = (IsShown(View) && bVisibleForSelection && bVisibleForShowFlags) || bShowForCollision;
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}

		virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
		uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

	private:
		const uint32				bDrawOnlyIfSelected:1;
		const FColor				SphereColor;
		const float					SphereRadius;
		const float					LineThickness;
	};

	return new FSphereSceneProxy( this );
}


FCollisionShape USphereComponent::GetCollisionShape(float Inflation) const
{
	const float Radius = GetScaledSphereRadius();
	return FCollisionShape::MakeSphere(Radius).Inflate(Inflation);
}

bool USphereComponent::AreSymmetricRotations(const FQuat& A, const FQuat& B, const FVector& Scale3D) const
{
	// All rotations are equal when scale is uniform.
	// Not detecting rotations around non-uniform scale.
	return Scale3D.GetAbs().AllComponentsEqual() || A.Equals(B);
}

