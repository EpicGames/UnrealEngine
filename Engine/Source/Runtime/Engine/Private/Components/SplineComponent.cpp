// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SplineComponent.h"
#include "Curves/Splines/TangentSpline.h"
#include "Engine/Engine.h"
#include "SceneView.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Math/RotationMatrix.h"
#include "MeshElementCollector.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveDrawingUtils.h"
#include "Algo/ForEach.h"
#include "Net/UnrealNetwork.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"

#include "LocalVertexFactory.h"
#include "SceneInterface.h"
#include "Materials/Material.h"
#include "DynamicMeshBuilder.h"
#include "StaticMeshResources.h"
#include "Misc/TransactionObjectEvent.h"
#include "UObject/ICookInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineComponent)

#if WITH_EDITOR
#include "Settings/LevelEditorViewportSettings.h"
#endif

#define SPLINE_FAST_BOUNDS_CALCULATION 0

DEFINE_LOG_CATEGORY_STATIC(LogSplineComponent, Log, All);

const FInterpCurveVector USplineComponent::DummyPointPosition = {};
const FInterpCurveQuat USplineComponent::DummyPointRotation = {};
const FInterpCurveVector USplineComponent::DummyPointScale = {};

namespace UE::SplineComponent
{

bool GUseSplineCurves = true;
FAutoConsoleVariableRef CVarUseSplineCurves(
	TEXT("SplineComponent.UseSplineCurves"),
	GUseSplineCurves,
	TEXT("When true, SplineCurves is the authoritative backing data.")
);

bool GValidateOnChange = false;
FAutoConsoleVariableRef CVarValidateOnChange(
	TEXT("SplineComponent.ValidateOnChange"),
	GValidateOnChange,
	TEXT("When true, the non-authoritative backing data is validated against the authoritative backing data when changing any data.")
);

bool GValidateOnLoad = false;
FAutoConsoleVariableRef CVarValidateOnLoad(
	TEXT("SplineComponent.ValidateOnLoad"),
	GValidateOnLoad,
	TEXT("When true, consistency between authoritative and non-authoritative backing data is validated at load time.")
);

bool GSynchronizeOnLoad = false;
FAutoConsoleVariableRef CVarSynchronizeOnLoad(
	TEXT("SplineComponent.SynchronizeOnLoad"),
	GSynchronizeOnLoad,
	TEXT("When true, consistency between authoritative and non-authoritative backing data is synchronized if inconsistent at load time.")
);
	
bool GValidateOnSave = false;
FAutoConsoleVariableRef CVarValidateOnSave(
	TEXT("SplineComponent.ValidateOnSave"),
	GValidateOnSave,
	TEXT("When true, consistency between authoritative and non-authoritative backing data is validated at save time.")
);

bool GFailedValidationEnsures = false;
FAutoConsoleVariableRef CVarFailedValidationEnsures(
	TEXT("SplineComponent.FailedValidationEnsures"),
	GFailedValidationEnsures,
	TEXT("When true, all validation checks will ensure if not passed.")
);

int32 GAddSplinePointImplementation = 1;
FAutoConsoleVariableRef CVarAddSplinePointImplementation(
	TEXT("SplineComponent.AddSplinePointImplementation"),
	GAddSplinePointImplementation,
	TEXT("0) Uses FSpline::AddPoint - 1) Uses FSpline::InsertPoint")
);

/**
 * This option exists because the tangent computation code for CurveClamped points is just not consistent.
 * I suspect the FSplineCurves logic has a bug, but this has not yet been confirmed.
 * This is not a major concern because auto-tangents can be considered 'transient' data because, while they are serialized,
 * they are overwritten when USplineComponent::UpdateSpline is invoked.
 */
bool GIgnoreCurveClampedPointTangentsForValidation = true;
	FAutoConsoleVariableRef CVarIgnoreCurveClampedPointTangentsForValidation(
	TEXT("SplineComponent.IgnoreCurveClampedPointTangentsForValidation"),
	GIgnoreCurveClampedPointTangentsForValidation,
	TEXT("When true, validation checks will not consider inconsistent tangents to be invalid when the point type is CurveClamped.")
);

int GForceLastAuthority = 0;
FAutoConsoleVariableRef CVarForceLastAuthority(
	TEXT("SplineComponent.ForceLastAuthority"),
	GForceLastAuthority,
	TEXT("If 1, synchronization on load will treat FSplineCurves as the authority. If 2, synchronization on load will tread FSpline as the authority.")
);

bool GOnlyUpdateActiveSpline = true;
FAutoConsoleVariableRef CVarOnlyUpdateActiveSpline(
	TEXT("SplineComponent.OnlyUpdateActiveSpline"),
	GOnlyUpdateActiveSpline,
	TEXT("When true, UpdateSpline will only call update on the current spline in use, based on ShouldUseSplineCurves(). Off by default in editor.")
);

static ELastAuthority GetEffectiveLastAuthority(const ELastAuthority InLastAuthority)
{
	ELastAuthority UseLastAuthority = InLastAuthority;
	switch (GForceLastAuthority)
	{
	case 1: UseLastAuthority = ELastAuthority::SplineCurves; break;
	case 2: UseLastAuthority = ELastAuthority::Spline; break;
	default: break;
	}
	return UseLastAuthority;
}
	
bool ShouldUseSplineCurvesStatic()
{
	return GUseSplineCurves || FSpline::GetDefaultImplementation() != UE::Spline::ESplineType::Tangent;
}

}

USplineMetadata::USplineMetadata(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool USplineComponent::Validate() const
{
	if (Spline.GetCurrentImplementation() != UE::Spline::ESplineType::Tangent)
	{
		// Can't be inconsistent if Spline is incapable of storing any data at all.
		return true;
	}

	const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();
	if (!ensureAlways(TangentSpline))
	{
		// Something is wrong...
		return false;
	}

	auto StringifySplinePointType = [](const ESplinePointType::Type InSplinePointType)
	{
		switch (InSplinePointType)
		{
		case ESplinePointType::Linear: return TEXT("Linear");
		case ESplinePointType::Curve: return TEXT("Curve");
		case ESplinePointType::Constant: return TEXT("Constant");
		case ESplinePointType::CurveClamped: return TEXT("CurveClamped");
		case ESplinePointType::CurveCustomTangent: return TEXT("CurveCustomTangent");
		default: return TEXT("Unknown");
		}
	};

	bool bValid = true;
	
	const int32 NumSplineCurvesPoints = WarninglessSplineCurves().Position.Points.Num();
	const int32 NumSplinePoints = TangentSpline->GetNumberOfPoints();
	const int32 NumPoints = FMath::Min(NumSplinePoints, NumSplineCurvesPoints);

	UE_LOGF(LogSplineComponent, Display, "Validating SplineComponent (%ls) - Authority: %ls", *GetPathName(), ShouldUseSplineCurves() ? TEXT("FSplineCurves") : TEXT("FSpline"));
	
	if (NumSplineCurvesPoints != NumSplinePoints)
	{
		UE_LOGF(LogSplineComponent, Display, "Internally inconsistent number of points (%d vs %d)!", NumSplineCurvesPoints, NumSplinePoints);
		bValid = false;
	}

	auto IsNearlyEqual = [](const double& A, const double& B, float RelTol, float AbsTol)
	{
		const float Tolerance = FMath::Max(AbsTol, RelTol * FMath::Max(FMath::Abs(A), FMath::Abs(B)));
		return FMath::IsNearlyEqual(A, B, Tolerance);
	};

	const float SplineCurvesLength = WarninglessSplineCurves().GetSplineLength();
	const float SplineLength = TangentSpline->GetDistanceAtParameter(TangentSpline->GetParameterSpace().Max);
	if (!IsNearlyEqual(SplineCurvesLength, SplineLength, UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER))
	{
		UE_LOGF(LogSplineComponent, Display, "Internally inconsistent spline lengths (%f vs %f)!", SplineCurvesLength, SplineLength);
		UE_LOGF(LogSplineComponent, Display, "Scale: %ls", *GetComponentTransform().GetScale3D().ToCompactString());
		
		bValid = false;
	}
	UE_LOGF(LogSplineComponent, Display, "Spline lengths: (%f vs %f)", SplineCurvesLength, SplineLength);

	for (int32 Idx = 0; Idx < NumPoints; ++Idx)
	{
		const auto& SplineCurvesPositionPoint = WarninglessSplineCurves().Position.Points[Idx];
		const auto& SplineCurvesRotationPoint = WarninglessSplineCurves().Rotation.Points[Idx];
		const auto& SplineCurvesScalePoint = WarninglessSplineCurves().Scale.Points[Idx];

		const float SplineCurvesParameter = SplineCurvesPositionPoint.InVal;
		const float SplineParameter = TangentSpline->GetParameterAtIndex(Idx);
		if (!IsNearlyEqual(SplineCurvesParameter, SplineParameter, UE_SMALL_NUMBER, UE_SMALL_NUMBER))
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent knot (%f vs %f) at index %d!", SplineCurvesParameter, SplineParameter, Idx );
			bValid = false;
		}

		const ESplinePointType::Type SplineCurvesPointType = ConvertInterpCurveModeToSplinePointType(SplineCurvesPositionPoint.InterpMode);
		const ESplinePointType::Type SplinePointType = ConvertInterpCurveModeToSplinePointType(TangentSpline->GetSplinePointType(Idx));
		if (SplineCurvesPointType != SplinePointType)
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent point type (%ls vs %ls) at index %d!", StringifySplinePointType(SplineCurvesPointType), StringifySplinePointType(SplinePointType), Idx );
			bValid = false;
		}
		
		const FVector SplineCurvesPosition = SplineCurvesPositionPoint.OutVal;
		const FVector SplinePosition = TangentSpline->GetLocation(Idx);
		if (!SplineCurvesPosition.Equals(SplinePosition))
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent position (%ls vs %ls) at index %d!", *SplineCurvesPosition.ToCompactString(), *SplinePosition.ToCompactString(), Idx );
			bValid = false;
		}

		const FVector SplineCurvesInTangent = SplineCurvesPositionPoint.ArriveTangent;
		const FVector SplineInTangent = TangentSpline->GetInTangent(Idx);
		if (!SplineCurvesInTangent.Equals(SplineInTangent))
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent arrive tangent (%ls vs %ls) at index %d!", *SplineCurvesInTangent.ToCompactString(), *SplineInTangent.ToCompactString(), Idx );
			UE_LOGF(LogSplineComponent, Display, "Point types: %ls and %ls", StringifySplinePointType(SplineCurvesPointType), StringifySplinePointType(SplinePointType));
			
			bool bWasValid = bValid;
			bValid = false;

			if (SplineCurvesPointType == ESplinePointType::CurveClamped && UE::SplineComponent::GIgnoreCurveClampedPointTangentsForValidation)
			{
				bValid = bWasValid;
				UE_LOGF(LogSplineComponent, Display, "Ignoring inconsistent arrive tangent for CurveClamped point.");
			}

			if (!IsClosedLoop() && Idx == 0 /* && UE::SplineComponent::GIgnoreEndpointInTangentMismatchForValidation */)
			{
				bValid = bWasValid;
				UE_LOGF(LogSplineComponent, Display, "Ignoring inconsistent arrive tangent for first point on open spline.");
			}
		}

		const FVector SplineCurvesOutTangent = SplineCurvesPositionPoint.LeaveTangent;
		const FVector SplineOutTangent = TangentSpline->GetOutTangent(Idx);
		if (!SplineCurvesOutTangent.Equals(SplineOutTangent))
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent leave tangent (%ls vs %ls) at index %d!", *SplineCurvesOutTangent.ToCompactString(), *SplineOutTangent.ToCompactString(), Idx );
			UE_LOGF(LogSplineComponent, Display, "Point types: %ls and %ls", StringifySplinePointType(SplineCurvesPointType), StringifySplinePointType(SplinePointType));
			
			bool bWasValid = bValid;
			bValid = false;

			if (SplineCurvesPointType == ESplinePointType::CurveClamped && UE::SplineComponent::GIgnoreCurveClampedPointTangentsForValidation)
			{
				bValid = bWasValid;
				UE_LOGF(LogSplineComponent, Display, "Ignoring inconsistent leave tangent for CurveClamped point.");
			}

			if (!IsClosedLoop() && Idx == GetNumberOfSplinePoints() - 1 /* && UE::SplineComponent::GIgnoreEndpointOutTangentMismatchForValidation */)
			{
				bValid = bWasValid;
				UE_LOGF(LogSplineComponent, Display, "Ignoring inconsistent leave tangent for last point on open spline.");
			}
		}

		const float SplineCurvesParam = SplineCurvesPositionPoint.InVal;
		const float SplineParam = TangentSpline->GetParameterAtIndex(Idx);
		if (!FMath::IsNearlyEqual(SplineCurvesParam, SplineParam))
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent parameter (%f vs %f) at index %d!", SplineCurvesParam, SplineParam, Idx );
			bValid = false;
		}

		const FQuat SplineCurvesRotation = SplineCurvesRotationPoint.OutVal;
		const FQuat SplineRotation = TangentSpline->GetRotation(Idx);
		if (!SplineCurvesRotation.Equals(SplineRotation))
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent rotation (%ls vs %ls) at index %d!", *SplineCurvesRotation.ToString(), *SplineRotation.ToString(), Idx );
			bValid = false;
		}

		const FVector SplineCurvesScale = SplineCurvesScalePoint.OutVal;
		const FVector SplineScale = TangentSpline->GetScale(Idx);
		if (!SplineCurvesScale.Equals(SplineScale))
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent scale (%ls vs %ls) at index %d!", *SplineCurvesScale.ToCompactString(), *SplineScale.ToCompactString(), Idx );
			bValid = false;
		}
	}

	if (UE::SplineComponent::GFailedValidationEnsures)
	{
		ensureAlways(bValid);
	}

	return bValid;
}

FName USplineComponent::GetSplinePropertyName()
{
	if (UE::SplineComponent::ShouldUseSplineCurvesStatic())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USplineComponent, SplineCurves);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		return GET_MEMBER_NAME_CHECKED(USplineComponent, Spline);
	}
}

TSet<FName> USplineComponent::GetSplinePropertyNames()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return {GET_MEMBER_NAME_CHECKED(USplineComponent, SplineCurves), GET_MEMBER_NAME_CHECKED(USplineComponent, Spline)};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

USplineComponent::USplineComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bAllowSplineEditingPerInstance_DEPRECATED(true)
#endif
	, ReparamStepsPerSegment(10)
	, Duration(1.0f)
	, bStationaryEndpoints(false)
	, bSplineHasBeenEdited(false)
	, bModifiedByConstructionScript(false)
	, bInputSplinePointsToConstructionScript(false)
	, bDrawDebug(true)
	, bClosedLoop(false)
	, DefaultUpVector(FVector::UpVector)
#if WITH_EDITORONLY_DATA
	, EditorUnselectedSplineSegmentColor(FStyleColors::White.GetSpecifiedColor())
	, EditorSelectedSplineSegmentColor(FStyleColors::AccentOrange.GetSpecifiedColor())
	, EditorTangentColor(FLinearColor(0.718f, 0.589f, 0.921f))
	, bAllowDiscontinuousSpline(false)
	, bAdjustTangentsOnSnap(true)
	, bShouldVisualizeScale(false)
	, ScaleVisualizationWidth(30.0f)
#endif
{
	SetCollisionEnabled(ECollisionEnabled::NoCollision);
	SetGenerateOverlapEvents(false);

	SetDefaultSpline();

#if WITH_EDITORONLY_DATA
	if (GEngine)
	{
		EditorSelectedSplineSegmentColor = GEngine->GetSelectionOutlineColor();
	}
#endif

	UpdateSpline();

#if WITH_EDITORONLY_DATA
	// Set these deprecated values up so that old assets with default values load correctly (and are subsequently upgraded during Serialize)
	SplineInfo_DEPRECATED = WarninglessSplineCurves().Position;
	SplineRotInfo_DEPRECATED = WarninglessSplineCurves().Rotation;
	SplineScaleInfo_DEPRECATED = WarninglessSplineCurves().Scale;
	SplineReparamTable_DEPRECATED = WarninglessSplineCurves().ReparamTable;
#endif

	// The default materials are soft object pointers so that they are not always loaded.
	LineMaterial = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/Engine/EngineMaterials/LineSetComponentMaterial.LineSetComponentMaterial")));
	PointMaterial = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/Engine/EngineMaterials/LineSetComponentMaterial.LineSetComponentMaterial")));
}

void USplineComponent::ResetToDefault()
{
	SetDefaultSpline();

	ReparamStepsPerSegment = 10;
	Duration = 1.0f;
	bStationaryEndpoints = false;
	bSplineHasBeenEdited = false;
	bModifiedByConstructionScript = false;
	bInputSplinePointsToConstructionScript = false;
	bDrawDebug  = true ;
	bClosedLoop = false;
	DefaultUpVector = FVector::UpVector;
#if WITH_EDITORONLY_DATA
	bAllowSplineEditingPerInstance_DEPRECATED = true;
	EditorUnselectedSplineSegmentColor = FStyleColors::White.GetSpecifiedColor();
	EditorSelectedSplineSegmentColor = FStyleColors::AccentOrange.GetSpecifiedColor();
	EditorTangentColor = FLinearColor(0.718f, 0.589f, 0.921f);
	bAllowDiscontinuousSpline = false;
	bShouldVisualizeScale = false;
	ScaleVisualizationWidth = 30.0f;
#endif

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
}

bool USplineComponent::CanResetToDefault() const
{
	return Spline != CastChecked<USplineComponent>(GetArchetype())->Spline;
}

FSpline USplineComponent::GetDefaultSpline()
{
	return StaticClass()->GetDefaultObject<USplineComponent>()->Spline;
}

void USplineComponent::SetDefaultSpline()
{
	if (UE::SplineComponent::GUseSplineCurves)
	{
		Spline = FSpline(UE::Spline::ESplineType::Unimplemented);
	}
	else
	{
		Spline = FSpline(UE::Spline::ESplineType::Tangent);
	}

	const FVector StartPoint(0.0, 0.0, 0.0);
	const FVector EndPoint(100.0, 0.0, 0.0);
	constexpr float StartParam = 0.f;
	constexpr float EndParam = 1.f;

	WarninglessSplineCurves().Position.Points.Reset(10);
    WarninglessSplineCurves().Rotation.Points.Reset(10);
    WarninglessSplineCurves().Scale.Points.Reset(10);

    WarninglessSplineCurves().Position.Points.Emplace(StartParam, StartPoint, FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
    WarninglessSplineCurves().Rotation.Points.Emplace(StartParam, FQuat::Identity, FQuat::Identity, FQuat::Identity, CIM_CurveAuto);
    WarninglessSplineCurves().Scale.Points.Emplace(StartParam, FVector(1.0f), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);

    WarninglessSplineCurves().Position.Points.Emplace(EndParam, EndPoint, FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
    WarninglessSplineCurves().Rotation.Points.Emplace(EndParam, FQuat::Identity, FQuat::Identity, FQuat::Identity, CIM_CurveAuto);
    WarninglessSplineCurves().Scale.Points.Emplace(EndParam, FVector(1.0f), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->Reset();
		TangentSpline->AddPoint(FSplinePoint(StartParam, StartPoint));
		TangentSpline->AddPoint(FSplinePoint(EndParam, EndPoint));
	}

	OnSplineChanged.Broadcast();

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
}

void USplineComponent::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	// This is a workaround for UE-129807 so that scrubbing a replay doesn't cause instance edited properties to be reset to class defaults
	// If you encounter this issue, reset the relevant replicated properties of this class with COND_ReplayOnly
	DISABLE_ALL_CLASS_REPLICATED_PROPERTIES(USplineComponent, EFieldIteratorFlags::ExcludeSuper);
}

EInterpCurveMode ConvertSplinePointTypeToInterpCurveMode(ESplinePointType::Type SplinePointType)
{
	switch (SplinePointType)
	{
		case ESplinePointType::Linear:				return CIM_Linear;
		case ESplinePointType::Curve:				return CIM_CurveAuto;
		case ESplinePointType::Constant:			return CIM_Constant;
		case ESplinePointType::CurveCustomTangent:	return CIM_CurveUser;
		case ESplinePointType::CurveClamped:		return CIM_CurveAutoClamped;

		default:									return CIM_Unknown;
	}
}

ESplinePointType::Type ConvertInterpCurveModeToSplinePointType(EInterpCurveMode InterpCurveMode)
{
	switch (InterpCurveMode)
	{
		case CIM_Linear:			return ESplinePointType::Linear;
		case CIM_CurveAuto:			return ESplinePointType::Curve;
		case CIM_Constant:			return ESplinePointType::Constant;
		case CIM_CurveUser:			return ESplinePointType::CurveCustomTangent;
		case CIM_CurveAutoClamped:	return ESplinePointType::CurveClamped;

		default:					return ESplinePointType::Constant;
	}
}

void USplineComponent::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		LastAuthority = ShouldUseSplineCurves() ? ELastAuthority::SplineCurves : ELastAuthority::Spline;
	}
	
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		// Move points into SplineCurves
		if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::SplineComponentCurvesInStruct)
		{
			WarninglessSplineCurves().Position = SplineInfo_DEPRECATED;
			WarninglessSplineCurves().Rotation = SplineRotInfo_DEPRECATED;
			WarninglessSplineCurves().Scale = SplineScaleInfo_DEPRECATED;
			WarninglessSplineCurves().ReparamTable = SplineReparamTable_DEPRECATED;
		}

		// Used to prevent redundant work.
		bool bHasSynchronized = false;

#if WITH_EDITOR
		// If SplineComponent.UseSplineCurves is false and Spline was not previously implemented
		// (implying it is not consistent with SplineCurves), initialize Spline from SplineCurves.
		if (Spline.GetPreviousImplementation() == UE::Spline::ESplineType::Unimplemented && !UE::SplineComponent::GUseSplineCurves)
		{
			Spline = WarninglessSplineCurves();
			bHasSynchronized = true;
		}
#endif

		auto Synchronize = [this, &bHasSynchronized]() -> void
		{
			switch (UE::SplineComponent::GetEffectiveLastAuthority(LastAuthority))
			{
			case ELastAuthority::Unset:	// Intentional fallthrough - at the time this property was added, the authority had been Spline for some time.
			case ELastAuthority::Spline:
				UE_LOGF(LogSplineComponent, Display, "Populating SplineCurves from Spline...")
					WarninglessSplineCurves() = FSplineCurves::FromSplineInterface(Spline);
				break;

			case ELastAuthority::SplineCurves:
				UE_LOGF(LogSplineComponent, Display, "Populating Spline from SplineCurves...")
					SetSpline(WarninglessSplineCurves());
				break;
			}

			UpdateSpline();

			bHasSynchronized = true;
		};

		// Previously, FTangentSpline's knot vector was always uniform, regardless of the input keys in SplineCurves.
		// Once our input key interface began propagating to the knot vector, it was necessary to synchronize them on load.
		if (!bHasSynchronized
			&& Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SplineComponentReparameterizeOnLoad)
		{
			if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
			{
				const int32 NumTangentSplinePoints = TangentSpline->GetNumberOfPoints();
				const int32 NumSplineCurvesPoints = WarninglessSplineCurves().Position.Points.Num();

				if (NumTangentSplinePoints != NumSplineCurvesPoints)
				{
					UE_LOGF(LogSplineComponent, Display, "Mismatched number of spline points, uniformly parameterizing TangentSpline.");
					UE_LOGF(LogSplineComponent, Display, "Path: %ls", *GetPathName());

					TangentSpline->Reparameterize(UE::Geometry::Spline::EParameterizationPolicy::Uniform);
				}
				else
				{
					TArray<float> SplineCurvesParameters;

					for (int32 SplineCurvesIdx = 0; SplineCurvesIdx < NumSplineCurvesPoints; ++SplineCurvesIdx)
					{
						SplineCurvesParameters.Add(WarninglessSplineCurves().Position.Points[SplineCurvesIdx].InVal);
					}

					TangentSpline->SetParameters(SplineCurvesParameters);

					// Changing parameterization invalidates loaded auto-tangents.
					FTangentSpline::FUpdateSplineParams Params{bClosedLoop, bStationaryEndpoints, ReparamStepsPerSegment, bLoopPositionOverride, LoopPosition, GetComponentTransform().GetScale3D()};
					TangentSpline->UpdateSpline(Params);
				}
			}
		}
		
		if (!bHasSynchronized
			&& UE::SplineComponent::GSynchronizeOnLoad
			&& Spline.GetCurrentImplementation() == UE::Spline::ESplineType::Tangent
			&& !Validate())
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent SplineComponent (%ls) at load time!", *GetPathName())

			Synchronize();
			
			// see if we got it right...
			if (!Validate())
			{
				UE_LOGF(LogSplineComponent, Warning, "Internally inconsistent SplineComponent (%ls) after synchronization on load!", *GetPathName())
			}
			else
			{
				UE_LOGF(LogSplineComponent, Display, "Successfully synchronized SplineComponent (%ls) on load.", *GetPathName())
			}
		}
		else if (UE::SplineComponent::GValidateOnLoad && Spline.GetCurrentImplementation() == UE::Spline::ESplineType::Tangent && !Validate())
		{
			UE_LOGF(LogSplineComponent, Warning, "Internally inconsistent spline data at load time!")
		}
	}
#endif

	if (Ar.IsSaving())
	{
		if (UE::SplineComponent::GValidateOnSave && !Validate())
		{
			UE_LOGF(LogSplineComponent, Display, "Internally inconsistent spline data at save time!")
		}
	}
	
	// Support old resources which don't have the rotation and scale splines present
	const FPackageFileVersion ArchiveUEVersion = Ar.UEVer();
	if (ArchiveUEVersion < VER_UE4_INTERPCURVE_SUPPORTS_LOOPING)
	{
		FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

		int32 LegacyNumPoints = WarninglessSplineCurves().Position.Points.Num();

		// The start point is no longer cloned as the endpoint when the spline is looped, so remove the extra endpoint if present
		if (bClosedLoop && ( GetLocationAtSplinePoint(0, ESplineCoordinateSpace::Local) == GetLocationAtSplinePoint(GetNumberOfSplinePoints() - 1, ESplineCoordinateSpace::Local)))
		{
			if (TangentSpline)
			{
				TangentSpline->RemovePoint(TangentSpline->GetNumberOfPoints() - 1);
			}
			WarninglessSplineCurves().Position.Points.RemoveAt(LegacyNumPoints - 1, EAllowShrinking::No);
			LegacyNumPoints--;
		}

		// Fill the other two splines with some defaults.
		
		WarninglessSplineCurves().Rotation.Points.Reset(LegacyNumPoints);
		WarninglessSplineCurves().Scale.Points.Reset(LegacyNumPoints);
		for (int32 Count = 0; Count < LegacyNumPoints; Count++)
		{
			WarninglessSplineCurves().Rotation.Points.Emplace(0.0f, FQuat::Identity, FQuat::Identity, FQuat::Identity, CIM_CurveAuto);
			WarninglessSplineCurves().Scale.Points.Emplace(0.0f, FVector(1.0f), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
		}
		
		// Importantly, these 2 functions guarantee that 1 rotation and scale value exists for each position,
		// but they are default values.
		if (TangentSpline)
		{
			TangentSpline->ResetRotation();
			TangentSpline->ResetScale();
		}

		UpdateSpline();
	}
}

void USplineComponent::PostLoad()
{
	Super::PostLoad();
}

void USplineComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITOR
	if (GetWorld() && !GetWorld()->IsGameWorld())
	{
		// Make sure the assigned TSoftObjectPtr material is loaded by that point so that we can create proxy render side.
		FCookLoadScope CookLoadScope(ECookLoadType::UsedInGame);
		(void)LineMaterial.LoadSynchronous();
		(void)PointMaterial.LoadSynchronous();
	}
#endif
}

FSplineCurves FSplineCurves::FromSplineInterface(const FSpline& InProxy)
{
	FSplineCurves NewCurve;

	if (const FTangentSpline* TangentSpline = InProxy.Get<UE::Spline::ESplineType::Tangent>())
	{
		NewCurve.Position = TangentSpline->GetSplinePointsPosition();
		NewCurve.Rotation = TangentSpline->GetSplinePointsRotation();
		NewCurve.Scale = TangentSpline->GetSplinePointsScale();

		// Populate LoopKeyOffset from the new spline when closed.
		if (TangentSpline->IsClosedLoop())
		{
			const int32 NumPoints = NewCurve.Position.Points.Num();
			if (NumPoints > 0)
			{
				const float LastInKey = NewCurve.Position.Points.Last().InVal;

				// FSpline already exposes this – SplineComponent uses it in GetInputKeyValueAtSplinePoint.
				const float ClosingParam = TangentSpline->GetParameterAtIndex(NumPoints);   // “closing” point
				const float LoopKeyOffset = ClosingParam - LastInKey;

				if (LoopKeyOffset > 0.f)
				{
					NewCurve.Position.bIsLooped    = true;
					NewCurve.Position.LoopKeyOffset = LoopKeyOffset;

					// Keep Rotation / Scale in sync
					NewCurve.Rotation.bIsLooped     = true;
					NewCurve.Rotation.LoopKeyOffset = LoopKeyOffset;
					NewCurve.Scale.bIsLooped        = true;
					NewCurve.Scale.LoopKeyOffset    = LoopKeyOffset;
				}
			}
		}
	}

	NewCurve.UpdateSpline(NewCurve.Position.bIsLooped, false);
	return NewCurve;
}

void FSplineCurves::UpdateSpline(bool bClosedLoop, bool bStationaryEndpoints, int32 ReparamStepsPerSegment, bool bLoopPositionOverride, float LoopPosition, const FVector& Scale3D)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSplineCurves::UpdateSpline);

	const int32 NumPoints = Position.Points.Num();
	check(Rotation.Points.Num() == NumPoints && Scale.Points.Num() == NumPoints);

#if DO_CHECK
	// Ensure input keys are strictly ascending
	for (int32 Index = 1; Index < NumPoints; Index++)
	{
		ensureAlways(Position.Points[Index - 1].InVal < Position.Points[Index].InVal);
	}
#endif

	// Ensure splines' looping status matches with that of the spline component
	if (bClosedLoop)
	{
		const float LastKey = Position.Points.Num() > 0 ? Position.Points.Last().InVal : 0.0f;
		const float LoopKey = bLoopPositionOverride ? LoopPosition : LastKey + 1.0f;
		Position.SetLoopKey(LoopKey);
		Rotation.SetLoopKey(LoopKey);
		Scale.SetLoopKey(LoopKey);
	}
	else
	{
		Position.ClearLoopKey();
		Rotation.ClearLoopKey();
		Scale.ClearLoopKey();
	}

	// Automatically set the tangents on any CurveAuto keys
	Position.AutoSetTangents(0.0f, bStationaryEndpoints);
	Rotation.AutoSetTangents(0.0f, bStationaryEndpoints);
	Scale.AutoSetTangents(0.0f, bStationaryEndpoints);

	// Now initialize the spline reparam table
	const int32 NumSegments = Position.bIsLooped ? NumPoints : FMath::Max(0, NumPoints - 1);

	// Start by clearing it
	ReparamTable.Points.Reset(NumSegments * ReparamStepsPerSegment + 1);
	float AccumulatedLength = 0.0f;
	for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		for (int32 Step = 0; Step < ReparamStepsPerSegment; ++Step)
		{
			const float Param = static_cast<float>(Step) / ReparamStepsPerSegment;
			const float SegmentLength = (Step == 0) ? 0.0f : GetSegmentLength(SegmentIndex, Param, Position.bIsLooped, Scale3D);

			ReparamTable.Points.Emplace(SegmentLength + AccumulatedLength, SegmentIndex + Param, 0.0f, 0.0f, CIM_Linear);
		}
		AccumulatedLength += GetSegmentLength(SegmentIndex, 1.0f, Position.bIsLooped, Scale3D);
	}
	ReparamTable.Points.Emplace(AccumulatedLength, static_cast<float>(NumSegments), 0.0f, 0.0f, CIM_Linear);
	++Version;
}

void USplineComponent::UpdateSpline()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USplineComponent::UpdateSpline);

	// If SplineComponent.UseSplineCurves is false and FSpline is not implemented, initialize it from SplineCurves.
	if (UE::SplineComponent::GUseSplineCurves == false && Spline.GetCurrentImplementation() != UE::Spline::ESplineType::Tangent)
	{
		Spline = WarninglessSplineCurves();
	}

	FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

	FTangentSpline::FUpdateSplineParams Params{bClosedLoop, bStationaryEndpoints, ReparamStepsPerSegment, bLoopPositionOverride, LoopPosition, GetComponentTransform().GetScale3D()};
	if (UE::SplineComponent::GOnlyUpdateActiveSpline)
	{
		if (ShouldUseSplineCurves())
		{
			WarninglessSplineCurves().UpdateSpline(Params.bClosedLoop, Params.bStationaryEndpoints, Params.ReparamStepsPerSegment, Params.bLoopPositionOverride, Params.LoopPosition, Params.Scale3D);
		}
		else if (ensureAlways(TangentSpline)) // ShouldUseSplineCurves should _never_ return false if TangentSpline would not be true
		{
			TangentSpline->UpdateSpline(Params);
		}
	}
	else
	{
		WarninglessSplineCurves().UpdateSpline(Params.bClosedLoop, Params.bStationaryEndpoints, Params.ReparamStepsPerSegment, Params.bLoopPositionOverride, Params.LoopPosition, Params.Scale3D);

		if (TangentSpline)
		{
			TangentSpline->UpdateSpline(Params);
		}
	}

	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, SplineCurves, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, bClosedLoop, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, bStationaryEndpoints, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, ReparamStepsPerSegment, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, bLoopPositionOverride, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, LoopPosition, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, DefaultUpVector, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, bSplineHasBeenEdited, this);
	MARK_PROPERTY_DIRTY_FROM_NAME(USplineComponent, bInputSplinePointsToConstructionScript, this);

#if UE_ENABLE_DEBUG_DRAWING
	if (bDrawDebug)
	{
		MarkRenderStateDirty();
	}
#endif

	OnSplineUpdated.Broadcast();

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
}

void USplineComponent::SetOverrideConstructionScript(bool bInOverride)
{
	bSplineHasBeenEdited = bInOverride;
}

float FSplineCurves::GetSegmentLength(const int32 Index, const float Param, bool bClosedLoop, const FVector& Scale3D) const
{
	const int32 NumPoints = Position.Points.Num();
	const int32 LastPoint = NumPoints - 1;

	check(Index >= 0 && ((bClosedLoop && Index < NumPoints) || (!bClosedLoop && Index < LastPoint)));
	check(Param >= 0.0f && Param <= 1.0f);

	// Evaluate the length of a Hermite spline segment.
	// This calculates the integral of |dP/dt| dt, where P(t) is the spline equation with components (x(t), y(t), z(t)).
	// This isn't solvable analytically, so we use a numerical method (Legendre-Gauss quadrature) which performs very well
	// with functions of this type, even with very few samples.  In this case, just 5 samples is sufficient to yield a
	// reasonable result.

	struct FLegendreGaussCoefficient
	{
		float Abscissa;
		float Weight;
	};

	static const FLegendreGaussCoefficient LegendreGaussCoefficients[] =
	{
		{ 0.0f, 0.5688889f },
		{ -0.5384693f, 0.47862867f },
		{ 0.5384693f, 0.47862867f },
		{ -0.90617985f, 0.23692688f },
		{ 0.90617985f, 0.23692688f }
	};

	const auto& StartPoint = Position.Points[Index];
	const auto& EndPoint = Position.Points[Index == LastPoint ? 0 : Index + 1];

	const auto& P0 = StartPoint.OutVal;
	const auto& T0 = StartPoint.LeaveTangent;
	const auto& P1 = EndPoint.OutVal;
	const auto& T1 = EndPoint.ArriveTangent;

	// Special cases for linear or constant segments
	if (StartPoint.InterpMode == CIM_Linear)
	{
		return ((P1 - P0) * Scale3D).Size() * Param;
	}
	else if (StartPoint.InterpMode == CIM_Constant)
	{
		// Special case: constant interpolation acts like distance = 0 for all p in [0, 1[ but for p == 1, the distance returned is the linear distance between start and end
		return Param == 1.f ? ((P1 - P0) * Scale3D).Size() : 0.0f;
	}

	// Cache the coefficients to be fed into the function to calculate the spline derivative at each sample point as they are constant.
	const FVector Coeff1 = ((P0 - P1) * 2.0f + T0 + T1) * 3.0f;
	const FVector Coeff2 = (P1 - P0) * 6.0f - T0 * 4.0f - T1 * 2.0f;
	const FVector Coeff3 = T0;

	const float HalfParam = Param * 0.5f;

	float Length = 0.0f;
	for (const auto& LegendreGaussCoefficient : LegendreGaussCoefficients)
	{
		// Calculate derivative at each Legendre-Gauss sample, and perform a weighted sum
		const float Alpha = HalfParam * (1.0f + LegendreGaussCoefficient.Abscissa);
		const FVector Derivative = ((Coeff1 * Alpha + Coeff2) * Alpha + Coeff3) * Scale3D;
		Length += Derivative.Size() * LegendreGaussCoefficient.Weight;
	}
	Length *= HalfParam;

	return Length;
}

float USplineComponent::GetSegmentLength(const int32 Index, const float Param) const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().GetSegmentLength(Index, Param, bClosedLoop, GetComponentTransform().GetScale3D());
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetSegmentLength(Index, Param, GetComponentTransform().GetScale3D());
	}

	ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	return 0.f;
}

float USplineComponent::GetSegmentParamFromLength(const int32 Index, const float Length, const float SegmentLength) const
{
	if (SegmentLength == 0.0f)
	{
		return 0.0f;
	}

	// Given a function P(x) which yields points along a spline with x = 0...1, we can define a function L(t) to be the
	// Euclidian length of the spline from P(0) to P(t):
	//
	//    L(t) = integral of |dP/dt| dt
	//         = integral of sqrt((dx/dt)^2 + (dy/dt)^2 + (dz/dt)^2) dt
	//
	// This method evaluates the inverse of this function, i.e. given a length d, it obtains a suitable value for t such that:
	//    L(t) - d = 0
	//
	// We use Newton-Raphson to iteratively converge on the result:
	//
	//    t' = t - f(t) / (df/dt)
	//
	// where: t is an initial estimate of the result, obtained through basic linear interpolation,
	//        f(t) is the function whose root we wish to find = L(t) - d,
	//        (df/dt) = d(L(t))/dt = |dP/dt|

	// TODO: check if this works OK with delta InVal != 1.0f
	// ^ it definitely does not

	int32 NumPoints = 0;
	if (ShouldUseSplineCurves())
	{
		NumPoints = WarninglessSplineCurves().Position.Points.Num();
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		NumPoints = TangentSpline->GetNumberOfPoints();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
	
	const int32 LastPoint = NumPoints - 1;

	check(Index >= 0 && ((bClosedLoop && Index < NumPoints) || (!bClosedLoop && Index < LastPoint)));
	check(Length >= 0.0f && Length <= SegmentLength);

	float Param = Length / SegmentLength;  // initial estimate for t

	// two iterations of Newton-Raphson is enough
	for (int32 Iteration = 0; Iteration < 2; ++Iteration)
	{
		float TangentMagnitude = ShouldUseSplineCurves()
			? WarninglessSplineCurves().Position.EvalDerivative(Index + Param, FVector::ZeroVector).Size()
			: Spline.EvaluateDerivative(Index + Param).Size();
		
		if (TangentMagnitude > 0.0f)
		{
			Param -= (GetSegmentLength(Index, Param) - Length) / TangentMagnitude;
			Param = FMath::Clamp(Param, 0.0f, 1.0f);
		}
	}

	return Param;
}

FVector USplineComponent::GetLocationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	FVector Location = ShouldUseSplineCurves()
		? WarninglessSplineCurves().Position.Eval(InKey, FVector::ZeroVector)
		: Spline.Evaluate(InKey);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Location = GetComponentTransform().TransformPosition(Location);
	}

	return Location;
}

FVector USplineComponent::GetTangentAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	FVector Tangent = ShouldUseSplineCurves()
		? WarninglessSplineCurves().Position.EvalDerivative(InKey, FVector::ZeroVector)
		: Spline.EvaluateDerivative(InKey);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Tangent = GetComponentTransform().TransformVector(Tangent);
	}

	return Tangent;
}

FVector USplineComponent::GetDirectionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	FVector Direction = ShouldUseSplineCurves()
		? WarninglessSplineCurves().Position.EvalDerivative(InKey, FVector::ZeroVector).GetSafeNormal()
		: Spline.EvaluateDerivative(InKey).GetSafeNormal();

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Direction = GetComponentTransform().TransformVector(Direction);
		Direction.Normalize();
	}

	return Direction;
}

FRotator USplineComponent::GetRotationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetQuaternionAtSplineInputKey(InKey, CoordinateSpace).Rotator();
}

FQuat USplineComponent::GetQuaternionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	FQuat Quat = FQuat::Identity;
	if (ShouldUseSplineCurves())
	{
		Quat = WarninglessSplineCurves().Rotation.Eval(InKey, FQuat::Identity);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Quat = TangentSpline->EvaluateRotation(InKey);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return Quat;
	}
	Quat.Normalize();

	FVector Direction = ShouldUseSplineCurves()
		? WarninglessSplineCurves().Position.EvalDerivative(InKey, FVector::ZeroVector).GetSafeNormal()
		: Spline.EvaluateDerivative(InKey).GetSafeNormal();
	
	const FVector UpVector = Quat.RotateVector(DefaultUpVector);

	FQuat Rot = (FRotationMatrix::MakeFromXZ(Direction, UpVector)).ToQuat();

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Rot = GetComponentTransform().GetRotation() * Rot;
	}

	return Rot;
}

FVector USplineComponent::GetUpVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const FQuat Quat = GetQuaternionAtSplineInputKey(InKey, ESplineCoordinateSpace::Local);
	FVector UpVector = Quat.RotateVector(FVector::UpVector);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		UpVector = GetComponentTransform().TransformVectorNoScale(UpVector);
	}

	return UpVector;
}

FVector USplineComponent::GetRightVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const FQuat Quat = GetQuaternionAtSplineInputKey(InKey, ESplineCoordinateSpace::Local);
	FVector RightVector = Quat.RotateVector(FVector::RightVector);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		RightVector = GetComponentTransform().TransformVectorNoScale(RightVector);
	}

	return RightVector;
}

FTransform USplineComponent::GetTransformAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale) const
{
	const FVector Location(GetLocationAtSplineInputKey(InKey, ESplineCoordinateSpace::Local));
	const FQuat Rotation(GetQuaternionAtSplineInputKey(InKey, ESplineCoordinateSpace::Local));
	const FVector Scale = bUseScale ? GetScaleAtSplineInputKey(InKey) : FVector(1.0f);

	FTransform Transform(Rotation, Location, Scale);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Transform = Transform * GetComponentTransform();
	}

	return Transform;
}

float USplineComponent::GetRollAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetRotationAtSplineInputKey(InKey, CoordinateSpace).Roll;
}


FVector USplineComponent::GetScaleAtSplineInputKey(float InKey) const
{
	FVector Scale = FVector::OneVector;

	if (ShouldUseSplineCurves())
	{
		Scale = WarninglessSplineCurves().Scale.Eval(InKey, FVector::OneVector);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Scale = TangentSpline->EvaluateScale(InKey);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Scale;
}

float USplineComponent::GetDistanceAlongSplineAtSplineInputKey(float InKey) const
{
	int32 NumPoints = 0;
	const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

	if (ShouldUseSplineCurves())
	{
		NumPoints = WarninglessSplineCurves().Position.Points.Num();
	}
	else if (TangentSpline)
	{
		NumPoints = TangentSpline->GetNumberOfPoints();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
	
	const int32 NumSegments = bClosedLoop ? NumPoints : NumPoints - 1;

	if ((InKey >= 0) && (InKey < NumSegments))
	{
		float Distance;
		
		// !TangentSpline is true only if ShouldUseSplineCurves previously returned true.
		if (!TangentSpline)
		{
			const int32 PointIndex = FMath::FloorToInt(InKey);
			const float Fraction = InKey - PointIndex;
			const int32 ReparamPointIndex = PointIndex * ReparamStepsPerSegment;
			Distance = WarninglessSplineCurves().ReparamTable.Points[ReparamPointIndex].InVal + GetSegmentLength(PointIndex, Fraction);
		}
		else
		{
			Distance = TangentSpline->GetDistanceAtParameter(InKey);
		}
		
		return Distance;
	}
	else if (InKey >= NumSegments)
	{
		return GetSplineLength();
	}

	return 0.0f;
}

float USplineComponent::GetDistanceAlongSplineAtLocation(const FVector& InLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const FVector LocalLocation = (CoordinateSpace == ESplineCoordinateSpace::World) ? GetComponentTransform().InverseTransformPosition(InLocation) : InLocation;
	float Dummy;
	
	float Key;
	if (ShouldUseSplineCurves())
	{
		Key = WarninglessSplineCurves().Position.FindNearest(LocalLocation, Dummy);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Key = TangentSpline->FindNearest(LocalLocation, Dummy);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
	
	return GetDistanceAlongSplineAtSplineInputKey(Key);
}

int32 USplineComponent::GetNumberOfPropertyValues(FName PropertyName) const
{
	if (ShouldUseSplineCurves() == false)
	{
		if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			return TangentSpline->NumAttributeValues<float>(PropertyName);
		}
	}

	return 0;
}

TArray<FName> USplineComponent::GetFloatPropertyChannels() const
{
	if (ShouldUseSplineCurves() == false)
	{
		if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			return TangentSpline->GetAttributeChannelNamesByValueType<float>();
		}
	}

	return TArray<FName>();
}

template<class T>
bool CreatePropertyChannel(const USplineMetadata* Metadata, FSpline& Spline, FName PropertyName)
{
	if (Metadata && Metadata->GetClass()->FindPropertyByName(PropertyName))
	{
		// can't create, it already exists on legacy metadata
		return false;
	}
	
	if (UE::SplineComponent::GUseSplineCurves == false)
	{
		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (!TangentSpline->HasAttributeChannel(PropertyName))
			{
				if (UE::Geometry::Spline::TTangentBezierSpline<T>* NewAttrChannel = TangentSpline->CreateAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<T>>(PropertyName))
				{
					NewAttrChannel->SetPreInfinityMode(UE::Geometry::Spline::EOutOfBoundsHandlingMode::Constant);
					NewAttrChannel->SetPostInfinityMode(UE::Geometry::Spline::EOutOfBoundsHandlingMode::Constant);

					return true;
				}
			}
		}
	}

	return false;
}

template<class T>
T GetPropertyAtSplineInputKey(const USplineMetadata* Metadata, const FSpline& Spline, float InKey, FName PropertyName)
{
	if (Metadata)
	{
		if (FProperty* Property = Metadata->GetClass()->FindPropertyByName(PropertyName))
		{
			const FInterpCurve<T>* Curve = Property->ContainerPtrToValuePtr<FInterpCurve<T>>(Metadata);
			return Curve->Eval(InKey, T(0));
		}
	}
	
	if (UE::SplineComponent::GUseSplineCurves == false)
	{
		if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (TangentSpline->HasAttributeChannel(PropertyName))
			{
				return TangentSpline->EvaluateAttribute<T>(PropertyName, InKey);
			}
		}
	}

	return T(0);
}

template<class T>
int32 AddPropertyAtSplineInputKey(FSpline& Spline, float InKey, const T& InValue, FName PropertyName)
{
	if (UE::SplineComponent::GUseSplineCurves == false)
	{
		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (TangentSpline->HasAttributeChannel(PropertyName))
			{
				return TangentSpline->AddAttributeValue<T>(PropertyName, InValue, InKey);
			}
		}
	}

	return INDEX_NONE;
}

float GetInputKeyAtIndex(const FSpline& Spline, int32 Index, FName PropertyName)
{
	if (UE::SplineComponent::GUseSplineCurves == false)
	{
		if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (TangentSpline->HasAttributeChannel(PropertyName))
			{
				return TangentSpline->GetAttributeParameter<float>(PropertyName, Index);
			}
		}
	}

	return 0.f;
}

int32 SetInputKeyAtIndex(FSpline& Spline, int32 Index, float InKey, FName PropertyName)
{
	if (UE::SplineComponent::GUseSplineCurves == false)
	{
		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (TangentSpline->HasAttributeChannel(PropertyName))
			{
				return TangentSpline->SetAttributeParameter<float>(PropertyName, Index, InKey);
			}
		}
	}

	return INDEX_NONE;
}

template<typename T>
float GetPropertyAtIndex(const FSpline& Spline, int32 Index, FName PropertyName)
{
	if (UE::SplineComponent::GUseSplineCurves == false)
	{
		if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (TangentSpline->HasAttributeChannel(PropertyName))
			{
				return TangentSpline->GetAttributeValue<T>(PropertyName, Index);
			}
		}
	}

	return 0.f;
}

template<typename T>
void SetPropertyAtIndex(FSpline& Spline, int32 Index, float Value, FName PropertyName)
{
	if (UE::SplineComponent::GUseSplineCurves == false)
	{
		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (TangentSpline->HasAttributeChannel(PropertyName))
			{
				TangentSpline->SetAttributeValue<T>(PropertyName, Value, Index);
			}
		}
	}
}

void USplineComponent::RemovePropertyAtIndex(int32 Index, FName PropertyName)
{
	if (ShouldUseSplineCurves() == false)
	{
		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (TangentSpline->HasAttributeChannel(PropertyName))
			{
				TangentSpline->RemoveAttributeValue<float>(PropertyName, Index);
			}
		}
	}
}

bool USplineComponent::SupportsAttributes() const
{
	// If GetAs yields a Tangent spline, we support attributes.
	if (ShouldUseSplineCurves() == false && Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return true;
	}

	return false;
}

bool USplineComponent::RemovePropertyChannel(FName PropertyName)
{
	if (ShouldUseSplineCurves() == false)
	{
		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			if (TangentSpline->HasAttributeChannel(PropertyName))
			{
				return TangentSpline->RemoveAttributeChannel(PropertyName);
			}
		}
	}

	return false;
}

void USplineComponent::CreateFloatPropertyChannel(FName PropertyName)
{
	CreatePropertyChannel<float>(GetSplinePointsMetadata(), Spline, PropertyName);
}

float USplineComponent::GetFloatPropertyAtSplineInputKey(float InKey, FName PropertyName) const
{
	return GetPropertyAtSplineInputKey<float>(GetSplinePointsMetadata(), Spline, InKey, PropertyName);
}

int32 USplineComponent::AddFloatPropertyAtSplineInputKey(float InKey, float Value, FName PropertyName)
{
	return AddPropertyAtSplineInputKey<float>(Spline, InKey, Value, PropertyName);
}

float USplineComponent::GetFloatPropertyInputKeyAtIndex(int32 Index, FName PropertyName) const
{
	return GetInputKeyAtIndex(Spline, Index, PropertyName);
}
	
int32 USplineComponent::SetFloatPropertyInputKeyAtIndex(int32 Index, float InputKey, FName PropertyName)
{
	return SetInputKeyAtIndex(Spline, Index, InputKey, PropertyName);
}
	
float USplineComponent::GetFloatPropertyAtIndex(int32 Index, FName PropertyName) const
{
	return GetPropertyAtIndex<float>(Spline, Index, PropertyName);
}
	
void USplineComponent::SetFloatPropertyAtIndex(int32 Index, float Value, FName PropertyName)
{
	SetPropertyAtIndex<float>(Spline, Index, Value, PropertyName);
}

void USplineComponent::SetClosedLoop(bool bInClosedLoop, bool bUpdateSpline)
{
	bClosedLoop = bInClosedLoop;
	bLoopPositionOverride = false;
	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	OnSplineChanged.Broadcast();

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
}

void USplineComponent::SetClosedLoopAtPosition(bool bInClosedLoop, float Key, bool bUpdateSpline)
{
	bClosedLoop = bInClosedLoop;
	bLoopPositionOverride = bInClosedLoop;
	LoopPosition = Key;

	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	OnSplineChanged.Broadcast();

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
}

bool USplineComponent::IsClosedLoop() const
{
	return bClosedLoop;
}

void USplineComponent::SetUnselectedSplineSegmentColor(const FLinearColor& Color)
{
#if WITH_EDITORONLY_DATA
	EditorUnselectedSplineSegmentColor = Color;
#endif
}


void USplineComponent::SetSelectedSplineSegmentColor(const FLinearColor& Color)
{
#if WITH_EDITORONLY_DATA
	EditorSelectedSplineSegmentColor = Color;
#endif
}

void USplineComponent::SetTangentColor(const FLinearColor& Color)
{
#if WITH_EDITORONLY_DATA
	EditorTangentColor = Color;
#endif
}

void USplineComponent::SetDrawDebug(bool bShow)
{
	bDrawDebug = bShow;
	MarkRenderStateDirty();
}

void USplineComponent::ClearSplinePoints(bool bUpdateSpline)
{
	Spline.Reset();

	WarninglessSplineCurves().Position.Points.Reset();
	WarninglessSplineCurves().Rotation.Points.Reset();
	WarninglessSplineCurves().Scale.Points.Reset();
	
	if (USplineMetadata* Metadata = GetSplinePointsMetadata())
	{
		Metadata->Reset(0);
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
}

static int32 UpperBound(const TArray<FInterpCurvePoint<FVector>>& SplinePoints, float Value)
{
	int32 Count = SplinePoints.Num();
	int32 First = 0;

	while (Count > 0)
	{
		const int32 Middle = Count / 2;
		if (Value >= SplinePoints[First + Middle].InVal)
		{
			First += Middle + 1;
			Count -= Middle + 1;
		}
		else
		{
			Count = Middle;
		}
	}

	return First;
}

void USplineComponent::AddPoint(const FSplinePoint& InSplinePoint, bool bUpdateSpline)
{
	int32 Index = UpperBound(WarninglessSplineCurves().Position.Points, InSplinePoint.InputKey);
		
	if (Index > 0 && FMath::IsNearlyEqual(InSplinePoint.InputKey, GetInputKeyValueAtSplinePoint(Index - 1)))
	{
		// Decrement in the case of a collision because we insert before the colliding element and increment that element's input key.
		Index--;

		// Increment keys only in the case of a collision to prevent invalid input keys.
		for (int I = GetNumberOfSplinePoints() - 1; I >= Index; --I)
		{
			SetInputKeyValueAtSplinePoint(I, GetInputKeyValueAtSplinePoint(I) + 1.f);
		}
	}
		
	WarninglessSplineCurves().Position.Points.Insert(FInterpCurvePoint<FVector>(
		InSplinePoint.InputKey,
		InSplinePoint.Position,
		InSplinePoint.ArriveTangent,
		InSplinePoint.LeaveTangent,
		ConvertSplinePointTypeToInterpCurveMode(InSplinePoint.Type)
	), Index);

	WarninglessSplineCurves().Rotation.Points.Insert(FInterpCurvePoint<FQuat>(
		InSplinePoint.InputKey,
		InSplinePoint.Rotation.Quaternion(),
		FQuat::Identity,
		FQuat::Identity,
		CIM_CurveAuto
	), Index);

	WarninglessSplineCurves().Scale.Points.Insert(FInterpCurvePoint<FVector>(
		InSplinePoint.InputKey,
		InSplinePoint.Scale,
		FVector::ZeroVector,
		FVector::ZeroVector,
		CIM_CurveAuto
	), Index);
	
	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->AddPoint(InSplinePoint);
	}
	
	if (USplineMetadata* Metadata = GetSplinePointsMetadata())
	{
		Metadata->AddPoint(InSplinePoint.InputKey);
	}

	const float LastPointKey = GetInputKeyValueAtSplinePoint(GetNumberOfSplinePoints() - 1);
	
	if (bLoopPositionOverride && LoopPosition <= LastPointKey)
	{
		bLoopPositionOverride = false;
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}

	OnSplineChanged.Broadcast();
}

void USplineComponent::AddPoints(const TArray<FSplinePoint>& InSplinePoints, bool bUpdateSpline)
{
	const int32 NumPoints = WarninglessSplineCurves().Position.Points.Num() + InSplinePoints.Num();
	// Position, Rotation, and Scale will all grow together.
	WarninglessSplineCurves().Position.Points.Reserve(NumPoints);
	WarninglessSplineCurves().Rotation.Points.Reserve(NumPoints);
	WarninglessSplineCurves().Scale.Points.Reserve(NumPoints);

	for (const auto& SplinePoint : InSplinePoints)
	{
		AddPoint(SplinePoint, false);
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
	
	OnSplineChanged.Broadcast();
}


void USplineComponent::AddSplinePoint(const FVector& Position, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline)
{
	const FVector TransformedPosition = (CoordinateSpace == ESplineCoordinateSpace::World) ?
		GetComponentTransform().InverseTransformPosition(Position) : Position;

	// Add the spline point at the end of the array, adding 1.0 to the current last input key.
	// This continues the former behavior in which spline points had to be separated by an interval of 1.0.
	const float InKey = GetNumberOfSplinePoints() > 0
		? GetInputKeyValueAtSplinePoint(GetNumberOfSplinePoints() - 1) + 1.0f
		: 0.0f;

	FSplinePoint NewPoint;
	NewPoint.InputKey = InKey;
	NewPoint.Position = TransformedPosition;
	NewPoint.ArriveTangent = FVector::ZeroVector;
	NewPoint.LeaveTangent = FVector::ZeroVector;
	NewPoint.Rotation = FQuat::Identity.Rotator();
	NewPoint.Scale = FVector(1.f);
	NewPoint.Type = ConvertInterpCurveModeToSplinePointType(CIM_CurveAuto);

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		switch (UE::SplineComponent::GAddSplinePointImplementation)
		{
		default: // fallthrough
		case 0: TangentSpline->AddPoint(NewPoint); break;
		case 1: TangentSpline->InsertPoint(NewPoint, GetNumberOfSplinePoints()); break;
		}
	}

	WarninglessSplineCurves().Position.Points.Emplace(InKey, TransformedPosition, FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
	WarninglessSplineCurves().Rotation.Points.Emplace(InKey, FQuat::Identity, FQuat::Identity, FQuat::Identity, CIM_CurveAuto);
	WarninglessSplineCurves().Scale.Points.Emplace(InKey, FVector(1.0f), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
	
	if (USplineMetadata* Metadata = GetSplinePointsMetadata())
	{
		Metadata->AddPoint(InKey);
	}

	if (bLoopPositionOverride)
	{
		LoopPosition += 1.0f;
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
	
	OnSplineChanged.Broadcast();
}

void USplineComponent::AddSplinePointAtIndex(const FVector& Position, int32 Index, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline)
{
	int32 NumPoints = GetNumberOfSplinePoints();
	
	const FVector TransformedPosition = (CoordinateSpace == ESplineCoordinateSpace::World) ?
		GetComponentTransform().InverseTransformPosition(Position) : Position;

	if (Index >= 0 && Index <= NumPoints)
	{
		const float InKey = (Index == 0) ? 0.0f : GetInputKeyValueAtSplinePoint(Index - 1) + 1.0f;

		FSplinePoint NewPoint;
		NewPoint.InputKey = InKey;
		NewPoint.Position = TransformedPosition;
		NewPoint.ArriveTangent = FVector::ZeroVector;
		NewPoint.LeaveTangent = FVector::ZeroVector;
		NewPoint.Rotation = FQuat::Identity.Rotator();
		NewPoint.Scale = FVector(1.f);

		FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

		// Adjust subsequent points' input keys to make room for the value we are about to add
		for (int I = NumPoints - 1; I >= Index; --I)
		{
			if (TangentSpline)
			{
				TangentSpline->SetParameterAtIndex(I, TangentSpline->GetParameterAtIndex(I) + 1.0f);
			}

			WarninglessSplineCurves().Position.Points[I].InVal += 1.0f;
			WarninglessSplineCurves().Rotation.Points[I].InVal += 1.0f;
			WarninglessSplineCurves().Scale.Points[I].InVal += 1.0f;
		}

		if (TangentSpline)
		{
			TangentSpline->InsertPoint(NewPoint, Index);
		}

		WarninglessSplineCurves().Position.Points.Insert(FInterpCurvePoint<FVector>(InKey, TransformedPosition, FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto), Index);
		WarninglessSplineCurves().Rotation.Points.Insert(FInterpCurvePoint<FQuat>(InKey, FQuat::Identity, FQuat::Identity, FQuat::Identity, CIM_CurveAuto), Index);
		WarninglessSplineCurves().Scale.Points.Insert(FInterpCurvePoint<FVector>(InKey, FVector(1.0f), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto), Index);
		
		if (USplineMetadata* Metadata = GetSplinePointsMetadata())
		{
			Metadata->InsertPoint(Index, 0.5f, bClosedLoop);
		}

		if (bLoopPositionOverride)
		{
			LoopPosition += 1.0f;
		}
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
	
	OnSplineChanged.Broadcast();
}

void USplineComponent::RemoveSplinePoint(int32 Index, bool bUpdateSpline)
{
	int32 NumPoints = GetNumberOfSplinePoints();

	if (Index >= 0 && Index < NumPoints)
	{
		FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

		float SplineCurvesKnotDelta = 0.0f;
		float SplineKnotDelta = 0.f;

		if (Index > 0)
		{
			SplineCurvesKnotDelta = WarninglessSplineCurves().Position.Points[Index].InVal - WarninglessSplineCurves().Position.Points[Index - 1].InVal;

			if (TangentSpline)
			{
				SplineKnotDelta = TangentSpline->GetParameterAtIndex(Index) - TangentSpline->GetParameterAtIndex(Index - 1);
			}
		}
		else if (NumPoints >= 2) // e.g. Index == 0, Index < NumPoints - 1
		{
			SplineCurvesKnotDelta = WarninglessSplineCurves().Position.Points[Index + 1].InVal - WarninglessSplineCurves().Position.Points[Index].InVal;

			if (TangentSpline)
			{
				SplineKnotDelta = TangentSpline->GetParameterAtIndex(Index + 1) - TangentSpline->GetParameterAtIndex(Index);
			}
		}

		if (TangentSpline)
		{
			TangentSpline->RemovePoint(Index);
		}
		
		WarninglessSplineCurves().Position.Points.RemoveAt(Index, EAllowShrinking::No);
		WarninglessSplineCurves().Rotation.Points.RemoveAt(Index, EAllowShrinking::No);
		WarninglessSplineCurves().Scale.Points.RemoveAt(Index, EAllowShrinking::No);
		
		if (USplineMetadata* Metadata = GetSplinePointsMetadata())
		{
			Metadata->RemovePoint(Index);
		}

		NumPoints--;

		// Adjust all following spline point input keys to close the gap left by the removed point
		while (Index < NumPoints)
		{
			if (TangentSpline)
			{
				TangentSpline->SetParameterAtIndex(Index, TangentSpline->GetParameterAtIndex(Index) - SplineKnotDelta);
			}

			WarninglessSplineCurves().Position.Points[Index].InVal -= SplineCurvesKnotDelta;
			WarninglessSplineCurves().Rotation.Points[Index].InVal -= SplineCurvesKnotDelta;
			WarninglessSplineCurves().Scale.Points[Index].InVal -= SplineCurvesKnotDelta;

			Index++;
		}

		if (bLoopPositionOverride)
		{
			LoopPosition -= 1.0f;
		}
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
	
	OnSplineChanged.Broadcast();
}

void USplineComponent::SetSplinePoints(const TArray<FVector>& Points, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline)
{
	const int32 NumPoints = Points.Num();

	WarninglessSplineCurves().Position.Points.Reset(NumPoints);
	WarninglessSplineCurves().Rotation.Points.Reset(NumPoints);
	WarninglessSplineCurves().Scale.Points.Reset(NumPoints);
	
	Spline.Reset();

	USplineMetadata* Metadata = GetSplinePointsMetadata();
	if (Metadata)
	{
		Metadata->Reset(NumPoints);
	}

	float InputKey = 0.0f;
	FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();
	for (const auto& Point : Points)
	{
		const FVector TransformedPoint = (CoordinateSpace == ESplineCoordinateSpace::World) ?
			GetComponentTransform().InverseTransformPosition(Point) : Point;

		FSplinePoint NewPoint;
		NewPoint.InputKey = InputKey;
		NewPoint.Position = TransformedPoint;
		NewPoint.ArriveTangent = FVector::ZeroVector;
		NewPoint.LeaveTangent = FVector::ZeroVector;
		NewPoint.Rotation = FQuat::Identity.Rotator();
		NewPoint.Scale = FVector(1.f);

		if (TangentSpline)
		{
			TangentSpline->AddPoint(NewPoint);
		}

		WarninglessSplineCurves().Position.Points.Emplace(InputKey, TransformedPoint, FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
		WarninglessSplineCurves().Rotation.Points.Emplace(InputKey, FQuat::Identity, FQuat::Identity, FQuat::Identity, CIM_CurveAuto);
		WarninglessSplineCurves().Scale.Points.Emplace(InputKey, FVector(1.0f), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);

		if (Metadata)
		{
			Metadata->AddPoint(InputKey);
		}

		InputKey += 1.0f;
	}

	bLoopPositionOverride = false;

	if (bUpdateSpline)
	{
		UpdateSpline();
	}

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}

	OnSplineChanged.Broadcast();
}

void USplineComponent::SetLocationAtSplinePoint(int32 PointIndex, const FVector& InLocation, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline)
{
	const int32 NumPoints = GetNumberOfSplinePoints();

	if ((PointIndex >= 0) && (PointIndex < NumPoints))
	{
		const FVector TransformedLocation = (CoordinateSpace == ESplineCoordinateSpace::World) ?
			GetComponentTransform().InverseTransformPosition(InLocation) : InLocation;

		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			TangentSpline->SetLocation(PointIndex, TransformedLocation);
		}

		WarninglessSplineCurves().Position.Points[PointIndex].OutVal = TransformedLocation;

		if (bUpdateSpline)
		{
			UpdateSpline();
		}

		if (UE::SplineComponent::GValidateOnChange)
		{
			Validate();
		}
		
		OnSplineChanged.Broadcast();
	}
}

void USplineComponent::SetTangentAtSplinePoint(int32 PointIndex, const FVector& InTangent, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline)
{
	SetTangentsAtSplinePoint(PointIndex, InTangent, InTangent, CoordinateSpace, bUpdateSpline);
}

void USplineComponent::SetTangentsAtSplinePoint(int32 PointIndex, const FVector& InArriveTangent, const FVector& InLeaveTangent, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline)
{
	const int32 NumPoints = GetNumberOfSplinePoints();

	if ((PointIndex >= 0) && (PointIndex < NumPoints))
	{
		const FVector TransformedArriveTangent = (CoordinateSpace == ESplineCoordinateSpace::World) ?
			GetComponentTransform().InverseTransformVector(InArriveTangent) : InArriveTangent;
		const FVector TransformedLeaveTangent = (CoordinateSpace == ESplineCoordinateSpace::World) ?
			GetComponentTransform().InverseTransformVector(InLeaveTangent) : InLeaveTangent;

		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			TangentSpline->SetInTangent(PointIndex, TransformedArriveTangent);
			TangentSpline->SetOutTangent(PointIndex, TransformedLeaveTangent);
		}

		WarninglessSplineCurves().Position.Points[PointIndex].ArriveTangent = TransformedArriveTangent;
		WarninglessSplineCurves().Position.Points[PointIndex].LeaveTangent = TransformedLeaveTangent;
		WarninglessSplineCurves().Position.Points[PointIndex].InterpMode = CIM_CurveUser;

		if (bUpdateSpline)
		{
			UpdateSpline();
		}

		if (UE::SplineComponent::GValidateOnChange)
		{
			Validate();
		}

		OnSplineChanged.Broadcast();
	}
}

void USplineComponent::SetUpVectorAtSplinePoint(int32 PointIndex, const FVector& InUpVector, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline)
{
	const int32 NumPoints = GetNumberOfSplinePoints();

	if ((PointIndex >= 0) && (PointIndex < NumPoints))
	{
		const FVector TransformedUpVector = (CoordinateSpace == ESplineCoordinateSpace::World) ?
			GetComponentTransform().InverseTransformVector(InUpVector.GetSafeNormal()) : InUpVector.GetSafeNormal();

		FQuat Quat = FQuat::FindBetween(DefaultUpVector, TransformedUpVector);

		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			TangentSpline->SetRotation(PointIndex, Quat);
		}

		WarninglessSplineCurves().Rotation.Points[PointIndex].OutVal = Quat;

		if (bUpdateSpline)
		{
			UpdateSpline();
		}

		if (UE::SplineComponent::GValidateOnChange)
		{
			Validate();
		}
		
		OnSplineChanged.Broadcast();
	}
}

void USplineComponent::SetRotationAtSplinePoint(int32 PointIndex, const FRotator& InRotation, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline /*= true*/)
{
	SetQuaternionAtSplinePoint(PointIndex, InRotation.Quaternion(), CoordinateSpace, bUpdateSpline);
}

void USplineComponent::SetQuaternionAtSplinePoint(int32 PointIndex, const FQuat& InQuaternion, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline)
{
	const int32 NumPoints = GetNumberOfSplinePoints();
	
	if ((PointIndex >= 0) && (PointIndex < NumPoints))
	{
		// InQuaternion coerced into local space. All subsequent operations in local space.
		const FQuat Quat = (CoordinateSpace == ESplineCoordinateSpace::World) ?
			GetComponentTransform().InverseTransformRotation(InQuaternion) : InQuaternion;

		// Work backwards to compute the rotation that is currently being applied.
		const FQuat RelativeQuat = Quat * GetQuaternionAtSplinePoint(PointIndex, ESplineCoordinateSpace::Local).Inverse();

		// Align up vector with rotation
		SetUpVectorAtSplinePoint(PointIndex, Quat.GetUpVector(), ESplineCoordinateSpace::Local, false);

		// Align tangents with rotation, preserving magnitude.
		const FVector OldArriveTangent = GetArriveTangentAtSplinePoint(PointIndex, ESplineCoordinateSpace::Local);
		const FVector OldArriveTangentDirection = OldArriveTangent.GetSafeNormal();
		const float ArriveTangentMag = OldArriveTangent.Length();
		const FVector NewArriveTangent = ArriveTangentMag * RelativeQuat.RotateVector(OldArriveTangentDirection);
		
		const FVector OldLeaveTangent = GetLeaveTangentAtSplinePoint(PointIndex, ESplineCoordinateSpace::Local);
		const FVector OldLeaveTangentDirection = OldLeaveTangent.GetSafeNormal();
		const float LeaveTangentMag = OldLeaveTangent.Length();
		const FVector NewLeaveTangent = LeaveTangentMag * RelativeQuat.RotateVector(OldLeaveTangentDirection);
				
		SetTangentsAtSplinePoint(PointIndex, NewArriveTangent, NewLeaveTangent, ESplineCoordinateSpace::Local, false);

		if (bUpdateSpline)
		{
			UpdateSpline();
		}

		// no need to validate, handled by SetTangentsAtSplinePoint
		
		OnSplineChanged.Broadcast();
	}
}

void USplineComponent::SetScaleAtSplinePoint(int32 PointIndex, const FVector& InScaleVector, bool bUpdateSpline /*= true*/)
{
	const int32 NumPoints = GetNumberOfSplinePoints();
	
	if ((PointIndex >= 0) && (PointIndex < NumPoints))
	{
		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			TangentSpline->SetScale(PointIndex, InScaleVector);
		}

		WarninglessSplineCurves().Scale.Points[PointIndex].OutVal = InScaleVector;

		if (bUpdateSpline)
		{
			UpdateSpline();
		}

		if (UE::SplineComponent::GValidateOnChange)
		{
			Validate();
		}

		OnSplineChanged.Broadcast();
	}
}

ESplinePointType::Type USplineComponent::GetSplinePointType(int32 PointIndex) const
{
	const int32 NumPoints = GetNumberOfSplinePoints();
	
	if (PointIndex >= 0 && PointIndex < NumPoints)
	{
		EInterpCurveMode Mode = CIM_Unknown;

		if (ShouldUseSplineCurves())
		{
			Mode = WarninglessSplineCurves().Position.Points[PointIndex].InterpMode.GetValue();
		}
		else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			Mode = TangentSpline->GetSplinePointType(PointIndex);
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		}
		
		return ConvertInterpCurveModeToSplinePointType(Mode);
	}

	return ESplinePointType::Constant;
}

void USplineComponent::SetSplinePointType(int32 PointIndex, ESplinePointType::Type Type, bool bUpdateSpline)
{
	const int32 NumPoints = GetNumberOfSplinePoints();
	
	if (PointIndex >= 0 && PointIndex < NumPoints)
	{
		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			TangentSpline->SetSplinePointType(PointIndex, ConvertSplinePointTypeToInterpCurveMode(Type));
		}

		WarninglessSplineCurves().Position.Points[PointIndex].InterpMode = ConvertSplinePointTypeToInterpCurveMode(Type);
		
		if (bUpdateSpline)
		{
			UpdateSpline();
		}

		if (UE::SplineComponent::GValidateOnChange)
		{
			Validate();
		}
		
		OnSplineChanged.Broadcast();
	}
}

int32 USplineComponent::GetNumberOfSplinePoints() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position.Points.Num();
	}
	if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetNumberOfPoints();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0;
	}
}

int32 USplineComponent::GetNumberOfSplineSegments() const
{
	const int32 NumPoints = GetNumberOfSplinePoints();

	if (NumPoints < 2)
	{
		return 0;
	}

	// We don't just simply read bClosedLoop because if LoopPosition holds an
	// invalid value the spline will be open regardless of the value of bClosedLoop.

	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position.bIsLooped ? NumPoints : FMath::Max(0, NumPoints - 1);
	}
	if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetNumberOfSegments();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0;
	}

}

float USplineComponent::GetInputKeyValueAtSplinePoint(int32 PointIndex) const
{
	const int32 NumPoints = GetNumberOfSplinePoints();

	if (NumPoints == 0)
	{
		return 0.f;
	}

	// Special case if we are closed and PointIndex refers to the closing point. For this function only we
	// allow users to treat it as a separate point in order to expose the parameterization of the closing segment.
	// The closing segment only exists if we have at least 2 points (otherwise there is no curve to close).
	if (bClosedLoop && NumPoints >= 2 && PointIndex == NumPoints)
	{
		if (ShouldUseSplineCurves())
		{
			if (!WarninglessSplineCurves().Position.bIsLooped)
			{
				// This is a weird case that can occur when LoopPosition is invalid
				return 0.f;
			}

			return WarninglessSplineCurves().Position.Points[PointIndex - 1].InVal + WarninglessSplineCurves().Position.LoopKeyOffset;
		}
		else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			return TangentSpline->GetParameterAtIndex(PointIndex);
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
			return 0.f;
		}
	}
	else
	{
		PointIndex = GetClampedIndex(PointIndex);

		if (ShouldUseSplineCurves())
		{
			return WarninglessSplineCurves().Position.Points[PointIndex].InVal;
		}
		else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			return TangentSpline->GetParameterAtIndex(PointIndex);
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
			return 0.f;
		}
	}
}

void USplineComponent::SetInputKeyValueAtSplinePoint(int32 PointIndex, float InputKey, bool bUpdateSpline)
{
	const int32 NumPoints = GetNumberOfSplinePoints();

	if (NumPoints == 0)
	{
		return;
	}

	PointIndex = GetClampedIndex(PointIndex);

	// Prevents reordering the knot vector. If the knot vector is monotonic, we only need to check the immediate neighbors of the point.
	if ((PointIndex + 1 < NumPoints && InputKey >= GetInputKeyValueAtSplinePoint(PointIndex + 1)) ||
		(PointIndex - 1 >= 0 && InputKey <= GetInputKeyValueAtSplinePoint(PointIndex - 1)))
	{
		UE_LOGF(LogSplineComponent, Warning, "Input Key change would reorder the spline, ignoring!");
		return;
	}

	WarninglessSplineCurves().Position.Points[PointIndex].InVal = InputKey;
	WarninglessSplineCurves().Rotation.Points[PointIndex].InVal = InputKey;
	WarninglessSplineCurves().Scale.Points[PointIndex].InVal = InputKey;

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->SetParameterAtIndex(PointIndex, InputKey);
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}
}

FSplinePoint USplineComponent::GetSplinePointAt(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	PointIndex = GetClampedIndex(PointIndex);

	return FSplinePoint(GetInputKeyValueAtSplinePoint(PointIndex),
		GetLocationAtSplinePoint(PointIndex, CoordinateSpace),
		GetArriveTangentAtSplinePoint(PointIndex, CoordinateSpace),
		GetLeaveTangentAtSplinePoint(PointIndex, CoordinateSpace),
		GetRotationAtSplinePoint(PointIndex, CoordinateSpace),
		GetScaleAtSplinePoint(PointIndex),
		GetSplinePointType(PointIndex));
}

FVector USplineComponent::GetLocationAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	if (GetNumberOfSplinePoints() == 0)
	{
		return FVector::ZeroVector; // from legacy DummyPointPosition
	}

	PointIndex = GetClampedIndex(PointIndex);
	
	FVector Location = FVector::Zero();
	if (ShouldUseSplineCurves())
	{
		Location = WarninglessSplineCurves().Position.Points[PointIndex].OutVal;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Location = TangentSpline->GetLocation(PointIndex);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return Location;
	}
	
	return (CoordinateSpace == ESplineCoordinateSpace::World) ? GetComponentTransform().TransformPosition(Location) : Location;
}

FVector USplineComponent::GetDirectionAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetTangentAtSplinePoint(PointIndex, CoordinateSpace).GetSafeNormal();
}

FVector USplineComponent::GetTangentAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetLeaveTangentAtSplinePoint(PointIndex, CoordinateSpace);
}

FVector USplineComponent::GetArriveTangentAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	if (GetNumberOfSplinePoints() == 0)
	{
		return FVector::ForwardVector; // from legacy DummyPointPosition
	}

	PointIndex = GetClampedIndex(PointIndex);

	FVector Tangent = FVector::ZeroVector;

	if (ShouldUseSplineCurves())
	{
		Tangent = WarninglessSplineCurves().Position.Points[PointIndex].ArriveTangent;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Tangent = TangentSpline->GetInTangent(PointIndex);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return Tangent;
	}

	return (CoordinateSpace == ESplineCoordinateSpace::World) ? GetComponentTransform().TransformVector(Tangent) : Tangent;
}

FVector USplineComponent::GetLeaveTangentAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	if (GetNumberOfSplinePoints() == 0)
	{
		return FVector::ForwardVector; // from legacy DummyPointPosition
	}

	PointIndex = GetClampedIndex(PointIndex);

	FVector Tangent = FVector::ZeroVector;

	if (ShouldUseSplineCurves())
	{
		Tangent = WarninglessSplineCurves().Position.Points[PointIndex].LeaveTangent;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Tangent = TangentSpline->GetOutTangent(PointIndex);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return Tangent;
	}

	return (CoordinateSpace == ESplineCoordinateSpace::World) ? GetComponentTransform().TransformVector(Tangent) : Tangent;
}

FQuat USplineComponent::GetQuaternionAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetQuaternionAtSplineInputKey(GetInputKeyValueAtSplinePoint(PointIndex), CoordinateSpace);
}

FRotator USplineComponent::GetRotationAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetRotationAtSplineInputKey(GetInputKeyValueAtSplinePoint(PointIndex), CoordinateSpace);
}

FVector USplineComponent::GetUpVectorAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetUpVectorAtSplineInputKey(GetInputKeyValueAtSplinePoint(PointIndex), CoordinateSpace);
}

FVector USplineComponent::GetRightVectorAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetRightVectorAtSplineInputKey(GetInputKeyValueAtSplinePoint(PointIndex), CoordinateSpace);
}

float USplineComponent::GetRollAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	return GetRollAtSplineInputKey(GetInputKeyValueAtSplinePoint(PointIndex), CoordinateSpace);
}

FVector USplineComponent::GetScaleAtSplinePoint(int32 PointIndex) const
{
	if (GetNumberOfSplinePoints() == 0)
	{
		return FVector::OneVector; // from legacy DummyPointScale
	}

	PointIndex = GetClampedIndex(PointIndex);

	FVector Scale = FVector::OneVector;

	if (ShouldUseSplineCurves())
	{
		Scale = WarninglessSplineCurves().Scale.Points[PointIndex].OutVal;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Scale = TangentSpline->GetScale(PointIndex);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Scale;
}

FTransform USplineComponent::GetTransformAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale) const
{
	return GetTransformAtSplineInputKey(GetInputKeyValueAtSplinePoint(PointIndex), CoordinateSpace, bUseScale);
}

void USplineComponent::GetLocationAndTangentAtSplinePoint(int32 PointIndex, FVector& Location, FVector& Tangent, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float InputKey = GetInputKeyValueAtSplinePoint(PointIndex);
	Location = GetLocationAtSplineInputKey(InputKey, CoordinateSpace);
	Tangent = GetTangentAtSplineInputKey(InputKey, CoordinateSpace);
}

float USplineComponent::GetDistanceAlongSplineAtSplinePoint(int32 PointIndex) const
{
	if (IsClosedLoop() && PointIndex == GetNumberOfSplinePoints())
	{
		// Special case, if we are closed and the index here is 1 past the last valid point, the length is the full spline.
		return GetSplineLength();
	}
	
	if (ShouldUseSplineCurves())
	{
		const int32 NumPoints = GetNumberOfSplinePoints();
        const int32 NumSegments = bClosedLoop ? NumPoints : NumPoints - 1;
        const int32 NumReparamPoints = WarninglessSplineCurves().ReparamTable.Points.Num();
        
        // Ensure that if the reparam table is not prepared yet we don't attempt to access it. This can happen
        // early in the construction of the spline component object.
        if ((PointIndex >= 0) && (PointIndex < NumSegments + 1) && ((PointIndex * ReparamStepsPerSegment) < NumReparamPoints))
        {
        	return WarninglessSplineCurves().ReparamTable.Points[PointIndex * ReparamStepsPerSegment].InVal;
        }
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		const float ParameterAtIndex = TangentSpline->GetParameterAtIndex(PointIndex);
		return TangentSpline->GetDistanceAtParameter(ParameterAtIndex);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return 0.0f;
}

float FSplineCurves::GetSplineLength() const
{
	const int32 NumPoints = ReparamTable.Points.Num();

	// This is given by the input of the last entry in the remap table
	if (NumPoints > 0)
	{
		return ReparamTable.Points.Last().InVal;
	}

	return 0.0f;
}

float USplineComponent::GetSplineLength() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().GetSplineLength();
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		const float MaxParameter = TangentSpline->GetParameterSpace().Max;
		return TangentSpline->GetDistanceAtParameter(MaxParameter);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

void USplineComponent::SetDefaultUpVector(const FVector& UpVector, ESplineCoordinateSpace::Type CoordinateSpace)
{
	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		DefaultUpVector = GetComponentTransform().InverseTransformVector(UpVector);
	}
	else
	{
		DefaultUpVector = UpVector;
	}

	UpdateSpline();

	OnSplineChanged.Broadcast();
}

FVector USplineComponent::GetDefaultUpVector(ESplineCoordinateSpace::Type CoordinateSpace) const
{
	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		return GetComponentTransform().TransformVector(DefaultUpVector);
	}
	else
	{
		return DefaultUpVector;
	}
}

float USplineComponent::GetInputKeyAtDistanceAlongSpline(float Distance) const
{
	return GetTimeAtDistanceAlongSpline(Distance);
}

float USplineComponent::GetInputKeyValueAtTime(float Time) const
{
	float T = Duration > 0.f ? FMath::Clamp(Time, 0.f, Duration) / Duration : 0.f;
	return GetInputKeyRange().Interpolate(T);
}

float USplineComponent::GetInputKeyValueAtDistanceAlongSpline(float Distance) const
{
	const int32 NumPoints = GetNumberOfSplinePoints();

	if (NumPoints < 2)
	{
		return 0.0f;
	}
	
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().ReparamTable.Eval(Distance, 0.0f);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetParameterAtDistance(Distance);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

float USplineComponent::GetTimeAtDistanceAlongSpline(float Distance) const
{
	const int32 NumPoints = GetNumberOfSplinePoints();

	if (NumPoints < 2)
	{
		return 0.0f;
	}

	float Param;
	if (ShouldUseSplineCurves())
	{
		Param = WarninglessSplineCurves().ReparamTable.Eval(Distance, 0.0f);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Param = TangentSpline->GetParameterAtDistance(Distance);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}

	// Convert Param to T and interpolate the duration interval (which yields Time).
	return UE::Geometry::FInterval1f(0.f, Duration).Interpolate(GetInputKeyRange().GetT(Param));
}

FVector USplineComponent::GetLocationAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetLocationAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::GetTangentAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetTangentAtSplineInputKey(Param, CoordinateSpace);
}


FVector USplineComponent::GetDirectionAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetDirectionAtSplineInputKey(Param, CoordinateSpace);
}

FQuat USplineComponent::GetQuaternionAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetQuaternionAtSplineInputKey(Param, CoordinateSpace);
}

FRotator USplineComponent::GetRotationAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetRotationAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::GetUpVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetUpVectorAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::GetRightVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetRightVectorAtSplineInputKey(Param, CoordinateSpace);
}

float USplineComponent::GetRollAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetRollAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::GetScaleAtDistanceAlongSpline(float Distance) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetScaleAtSplineInputKey(Param);
}

FTransform USplineComponent::GetTransformAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale) const
{
	const float Param = GetInputKeyValueAtDistanceAlongSpline(Distance);
	return GetTransformAtSplineInputKey(Param, CoordinateSpace, bUseScale);
}

FVector USplineComponent::GetLocationAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return FVector::ZeroVector;
	}

	if (bUseConstantVelocity)
	{
		return GetLocationAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace);
	}
	else
	{
		return GetLocationAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace);
	}
}

FVector USplineComponent::GetDirectionAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return FVector::ZeroVector;
	}

	if (bUseConstantVelocity)
	{
		return GetDirectionAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace);
	}
	else
	{
		return GetDirectionAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace);
	}
}

FVector USplineComponent::GetTangentAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return FVector::ZeroVector;
	}

	if (bUseConstantVelocity)
	{
		return GetTangentAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace);
	}
	else
	{
		return GetTangentAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace);
	}
}

FRotator USplineComponent::GetRotationAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return FRotator::ZeroRotator;
	}

	if (bUseConstantVelocity)
	{
		return GetRotationAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace);
	}
	else
	{
		return GetRotationAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace);
	}
}

FQuat USplineComponent::GetQuaternionAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return FQuat::Identity;
	}

	if (bUseConstantVelocity)
	{
		return GetQuaternionAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace);
	}
	else
	{
		return GetQuaternionAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace);
	}
}

FVector USplineComponent::GetUpVectorAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return FVector::ZeroVector;
	}

	if (bUseConstantVelocity)
	{
		return GetUpVectorAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace);
	}
	else
	{
		return GetUpVectorAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace);
	}
}

FVector USplineComponent::GetRightVectorAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return FVector::ZeroVector;
	}

	if (bUseConstantVelocity)
	{
		return GetRightVectorAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace);
	}
	else
	{
		return GetRightVectorAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace);
	}
}

float USplineComponent::GetRollAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return 0.0f;
	}

	if (bUseConstantVelocity)
	{
		return GetRollAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace);
	}
	else
	{
		return GetRollAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace);
	}
}

FTransform USplineComponent::GetTransformAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity, bool bUseScale) const
{
	if (Duration == 0.0f)
	{
		return FTransform::Identity;
	}

	if (bUseConstantVelocity)
	{
		return GetTransformAtDistanceAlongSpline(Time / Duration * GetSplineLength(), CoordinateSpace, bUseScale);
	}
	else
	{
		return GetTransformAtSplineInputKey(GetInputKeyValueAtTime(Time), CoordinateSpace, bUseScale);
	}
}

FVector USplineComponent::GetScaleAtTime(float Time, bool bUseConstantVelocity) const
{
	if (Duration == 0.0f)
	{
		return FVector(1.0f);
	}

	if (bUseConstantVelocity)
	{
		return GetScaleAtDistanceAlongSpline(Time / Duration * GetSplineLength());
	}
	else
	{
		return GetScaleAtSplineInputKey(GetInputKeyValueAtTime(Time));
	}
}

float USplineComponent::FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const
{
	const FVector LocalLocation = GetComponentTransform().InverseTransformPosition(WorldLocation);
	float Dummy;

	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position.FindNearest(LocalLocation, Dummy);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->FindNearest(LocalLocation, Dummy);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

float USplineComponent::FindInputKeyOnSegmentClosestToWorldLocation(const FVector& WorldLocation, int32 Index) const
{
	const FVector LocalLocation = GetComponentTransform().InverseTransformPosition(WorldLocation);
	float Dummy;

	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position.FindNearestOnSegment(LocalLocation, Index, Dummy);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->FindNearestOnSegment(LocalLocation, Index, Dummy);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

FVector USplineComponent::FindLocationClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetLocationAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::FindDirectionClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetDirectionAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::FindTangentClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetTangentAtSplineInputKey(Param, CoordinateSpace);
}

FQuat USplineComponent::FindQuaternionClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetQuaternionAtSplineInputKey(Param, CoordinateSpace);
}

FRotator USplineComponent::FindRotationClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetRotationAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::FindUpVectorClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetUpVectorAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::FindRightVectorClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetRightVectorAtSplineInputKey(Param, CoordinateSpace);
}

float USplineComponent::FindRollClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetRollAtSplineInputKey(Param, CoordinateSpace);
}

FVector USplineComponent::FindScaleClosestToWorldLocation(const FVector& WorldLocation) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetScaleAtSplineInputKey(Param);
}

FTransform USplineComponent::FindTransformClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale) const
{
	const float Param = FindInputKeyClosestToWorldLocation(WorldLocation);
	return GetTransformAtSplineInputKey(Param, CoordinateSpace, bUseScale);
}

bool USplineComponent::DivideSplineIntoPolylineRecursiveWithDistances(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline) const
{
	return ConvertSplineToPolyline_InDistanceRange(CoordinateSpace, MaxSquareDistanceFromSpline, StartDistanceAlongSpline, EndDistanceAlongSpline, OutPoints, OutDistancesAlongSpline, false);
}

bool USplineComponent::DivideSplineIntoPolylineRecursiveWithDistancesHelper(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline) const
{
	double Dist = EndDistanceAlongSpline - StartDistanceAlongSpline;
	if (Dist <= 0.0f)
	{
		return false;
	}
	double MiddlePointDistancAlongSpline = StartDistanceAlongSpline + Dist / 2.0f;
	FVector Samples[3];
	Samples[0] = GetLocationAtDistanceAlongSpline(StartDistanceAlongSpline, CoordinateSpace);
	Samples[1] = GetLocationAtDistanceAlongSpline(MiddlePointDistancAlongSpline, CoordinateSpace);
	Samples[2] = GetLocationAtDistanceAlongSpline(EndDistanceAlongSpline, CoordinateSpace);

	if (FMath::PointDistToSegmentSquared(Samples[1], Samples[0], Samples[2]) > MaxSquareDistanceFromSpline)
	{
		TArray<FVector> NewPoints[2];
		TArray<double> NewDistancesAlongSpline[2];
		DivideSplineIntoPolylineRecursiveWithDistancesHelper(StartDistanceAlongSpline, MiddlePointDistancAlongSpline, CoordinateSpace, MaxSquareDistanceFromSpline, NewPoints[0], NewDistancesAlongSpline[0]);
		DivideSplineIntoPolylineRecursiveWithDistancesHelper(MiddlePointDistancAlongSpline, EndDistanceAlongSpline, CoordinateSpace, MaxSquareDistanceFromSpline, NewPoints[1], NewDistancesAlongSpline[1]);
		if ((NewPoints[0].Num() > 0) && (NewPoints[1].Num() > 0))
		{
			check(NewPoints[0].Last() == NewPoints[1][0]);
			check(NewDistancesAlongSpline[0].Last() == NewDistancesAlongSpline[1][0]);
			NewPoints[0].RemoveAt(NewPoints[0].Num() - 1);
			NewDistancesAlongSpline[0].RemoveAt(NewDistancesAlongSpline[0].Num() - 1);
		}
		NewPoints[0].Append(NewPoints[1]);
		NewDistancesAlongSpline[0].Append(NewDistancesAlongSpline[1]);
		OutPoints.Append(NewPoints[0]);
		OutDistancesAlongSpline.Append(NewDistancesAlongSpline[0]);
	}
	else
	{
		// The middle point is close enough to the other 2 points, let's keep those and stop the recursion :
		OutPoints.Add(Samples[0]);
		OutDistancesAlongSpline.Add(StartDistanceAlongSpline);
		// For a constant spline, the end can be the exact same as the start; in this case, just add the point once
		if (Samples[0] != Samples[2])
		{
			OutPoints.Add(Samples[2]);
			OutDistancesAlongSpline.Add(EndDistanceAlongSpline);
		}
		
	}

	check(OutPoints.Num() == OutDistancesAlongSpline.Num())
	return (OutPoints.Num() > 0);
}

bool USplineComponent::DivideSplineIntoPolylineRecursiveHelper(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const
{
	TArray<double> DummyDistancesAlongSpline;
	return DivideSplineIntoPolylineRecursiveWithDistancesHelper(StartDistanceAlongSpline, EndDistanceAlongSpline, CoordinateSpace, MaxSquareDistanceFromSpline, OutPoints, DummyDistancesAlongSpline);
}

bool USplineComponent::DivideSplineIntoPolylineRecursive(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const
{
	TArray<double> DummyDistancesAlongSpline;
	return ConvertSplineToPolyline_InDistanceRange(CoordinateSpace, MaxSquareDistanceFromSpline, StartDistanceAlongSpline, EndDistanceAlongSpline, OutPoints, DummyDistancesAlongSpline, false);
}

bool USplineComponent::ConvertSplineSegmentToPolyLine(int32 SplinePointStartIndex, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const
{
	OutPoints.Empty();

	const double StartDist = GetDistanceAlongSplineAtSplinePoint(SplinePointStartIndex);
	const double StopDist = GetDistanceAlongSplineAtSplinePoint(SplinePointStartIndex + 1);

	const int32 NumLines = 2; // Dichotomic subdivision of the spline segment
	double Dist = StopDist - StartDist;
	double SubstepSize = Dist / NumLines;
	if (SubstepSize == 0.0)
	{
		// There is no distance to cover, so handle the segment with a single point
		OutPoints.Add(GetLocationAtDistanceAlongSpline(StopDist, CoordinateSpace));
		return true;
	}

	double SubstepStartDist = StartDist;
	for (int32 i = 0; i < NumLines; ++i)
	{
		double SubstepEndDist = SubstepStartDist + SubstepSize;
		TArray<FVector> NewPoints;
		// Recursively sub-divide each segment until the requested precision is reached :
		if (DivideSplineIntoPolylineRecursiveHelper(SubstepStartDist, SubstepEndDist, CoordinateSpace, MaxSquareDistanceFromSpline, NewPoints))
		{
			if (OutPoints.Num() > 0)
			{
				check(OutPoints.Last() == NewPoints[0]); // our last point must be the same as the new segment's first
				OutPoints.RemoveAt(OutPoints.Num() - 1);
			}
			OutPoints.Append(NewPoints);
		}

		SubstepStartDist = SubstepEndDist;
	}

	return (OutPoints.Num() > 0);
}

bool USplineComponent::ConvertSplineToPolyLine(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const
{
	int32 NumSegments = GetNumberOfSplineSegments();
	OutPoints.Empty();
	OutPoints.Reserve(NumSegments * 2); // We sub-divide each segment in at least 2 sub-segments, so let's start with this amount of points

	TArray<FVector> SegmentPoints;
	for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		if (ConvertSplineSegmentToPolyLine(SegmentIndex, CoordinateSpace, MaxSquareDistanceFromSpline, SegmentPoints))
		{
			if (OutPoints.Num() > 0)
			{
				check(OutPoints.Last() == SegmentPoints[0]); // our last point must be the same as the new segment's first
				OutPoints.RemoveAt(OutPoints.Num() - 1);
			}
			OutPoints.Append(SegmentPoints);
		}
	}

	return (OutPoints.Num() > 0);
}

bool USplineComponent::ConvertSplineToPolyLineWithDistances(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline) const
{
	return ConvertSplineToPolyline_InDistanceRange(CoordinateSpace, MaxSquareDistanceFromSpline, 0, GetSplineLength(), OutPoints, OutDistancesAlongSpline, false);
}

bool USplineComponent::ConvertSplineToPolyline_InDistanceRange(ESplineCoordinateSpace::Type CoordinateSpace, const float InMaxSquareDistanceFromSpline, float RangeStart, float RangeEnd, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline, bool bAllowWrappingIfClosed) const
{
	const int32 NumPoints = GetNumberOfSplinePoints();
	if (NumPoints == 0)
	{
		return false;
	}
	const int32 NumSegments = GetNumberOfSplineSegments();

	float SplineLength = GetSplineLength();
	if (SplineLength <= 0)
	{
		OutPoints.Add(GetLocationAtDistanceAlongSpline(0, CoordinateSpace));
		OutDistancesAlongSpline.Add(0);
		return false;
	}

	// Sanitize the sampling tolerance
	const float MaxSquareDistanceFromSpline = FMath::Max(UE_SMALL_NUMBER, InMaxSquareDistanceFromSpline);

	// Sanitize range and mark whether the range wraps through 0
	bool bNeedsWrap = false;
	if (!bClosedLoop || !bAllowWrappingIfClosed)
	{
		RangeStart = FMath::Clamp(RangeStart, 0, SplineLength);
		RangeEnd = FMath::Clamp(RangeEnd, 0, SplineLength);
	}
	else if (RangeStart < 0 || RangeEnd > SplineLength)
	{
		bNeedsWrap = true;
	}
	if (RangeStart > RangeEnd)
	{
		return false;
	}

	// expect at least 2 points per segment covered
	int32 EstimatedPoints = 2 * NumSegments * static_cast<int32>((RangeEnd - RangeStart) / SplineLength);
	OutPoints.Empty();
	OutPoints.Reserve(EstimatedPoints);
	OutDistancesAlongSpline.Empty();
	OutDistancesAlongSpline.Reserve(EstimatedPoints);

	if (RangeStart == RangeEnd)
	{
		OutPoints.Add(GetLocationAtDistanceAlongSpline(RangeStart, CoordinateSpace));
		OutDistancesAlongSpline.Add(RangeStart);
		return true;
	}

	// If we need to wrap around, break the wrapped segments into non-wrapped parts and add each part separately
	if (bNeedsWrap)
	{
		float TotalRange = RangeEnd - RangeStart;
		auto WrapDistance = [SplineLength](float Distance, int32& LoopIdx) -> float
		{
			LoopIdx = FMath::FloorToInt32(Distance / SplineLength);
			float WrappedDistance = FMath::Fmod(Distance, SplineLength);
			if (WrappedDistance < 0)
			{
				WrappedDistance += SplineLength;
			}
			return WrappedDistance;
		};
		int32 StartLoopIdx, EndLoopIdx;
		float WrappedStart = WrapDistance(RangeStart, StartLoopIdx);
		float WrappedEnd = WrapDistance(RangeEnd, EndLoopIdx);
		float WrappedLoc = WrappedStart;
		bool bHasAdded = false;
		for (int32 LoopIdx = StartLoopIdx; LoopIdx <= EndLoopIdx; ++LoopIdx)
		{
			if (bHasAdded && ensure(OutPoints.Num()))
			{
				OutPoints.RemoveAt(OutPoints.Num() - 1, EAllowShrinking::No);
				OutDistancesAlongSpline.RemoveAt(OutDistancesAlongSpline.Num() - 1, EAllowShrinking::No);
			}
			float EndLoc = LoopIdx == EndLoopIdx ? WrappedEnd : SplineLength;

			TArray<FVector> Points;
			TArray<double> Distances;
			ConvertSplineToPolyline_InDistanceRange(CoordinateSpace, MaxSquareDistanceFromSpline, WrappedLoc, EndLoc, Points, Distances, false);
			OutPoints.Append(Points);
			OutDistancesAlongSpline.Append(Distances);

			bHasAdded = true;
			WrappedLoc = 0;
		}
		return bHasAdded;
	} // end of the wrap-around case, after this values will be in the normal range
	
	int32 SegmentStart;
	int32 SegmentEnd;
	if (ShouldUseSplineCurves())
	{
		int32 StartIndex = WarninglessSplineCurves().ReparamTable.GetPointIndexForInputValue(RangeStart);
		int32 EndIndex = WarninglessSplineCurves().ReparamTable.GetPointIndexForInputValue(RangeEnd);
		SegmentStart = StartIndex / ReparamStepsPerSegment;
		SegmentEnd = FMath::Min(NumSegments, 1 + EndIndex / ReparamStepsPerSegment);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		const float RangeStartParam = GetInputKeyValueAtDistanceAlongSpline(RangeStart);
		const float RangeEndParam = GetInputKeyValueAtDistanceAlongSpline(RangeEnd);

		SegmentStart = TangentSpline->GetIndexAtParameter(RangeStartParam);
		SegmentEnd = FMath::Min(NumSegments, 1 + TangentSpline->GetIndexAtParameter(RangeEndParam));
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return false;
	}

	TArray<FVector> NewPoints;
	TArray<double> NewDistances;
	for (int32 SegmentIndex = SegmentStart; SegmentIndex < SegmentEnd; ++SegmentIndex)
	{
		if (SegmentIndex == INDEX_NONE)
		{
			continue;
		}

		// Get the segment range as distances, clipped with the input range
		double StartDist = FMath::Max(RangeStart, GetDistanceAlongSplineAtSplinePoint(SegmentIndex));
		double StopDist = FMath::Min(RangeEnd, GetDistanceAlongSplineAtSplinePoint(SegmentIndex + 1));
		bool bIsLast = SegmentIndex + 1 == SegmentEnd;

		const int32 NumLines = 2; // Dichotomic subdivision of the spline segment
		double Dist = StopDist - StartDist;
		double SubstepSize = Dist / NumLines;
		if (SubstepSize == 0.0)
		{
			// There is no distance to cover, so handle the segment with a single point (or nothing, if this isn't the very last point)
			if (bIsLast)
			{
				OutPoints.Add(GetLocationAtDistanceAlongSpline(StopDist, CoordinateSpace));
				OutDistancesAlongSpline.Add(StopDist);
			}
			continue;
		}

		double SubstepStartDist = StartDist;
		for (int32 i = 0; i < NumLines; ++i)
		{
			double SubstepEndDist = SubstepStartDist + SubstepSize;
			NewPoints.Reset();
			NewDistances.Reset();
			// Recursively sub-divide each segment until the requested precision is reached :
			if (DivideSplineIntoPolylineRecursiveWithDistancesHelper(SubstepStartDist, SubstepEndDist, CoordinateSpace, MaxSquareDistanceFromSpline, NewPoints, NewDistances))
			{
				if (OutPoints.Num() > 0)
				{
					check(OutPoints.Last() == NewPoints[0]); // our last point must be the same as the new segment's first
					OutPoints.RemoveAt(OutPoints.Num() - 1);
					OutDistancesAlongSpline.RemoveAt(OutDistancesAlongSpline.Num() - 1);
				}
				OutPoints.Append(NewPoints);
				OutDistancesAlongSpline.Append(NewDistances);
			}

			SubstepStartDist = SubstepEndDist;
		}
	}

	return !OutPoints.IsEmpty();
}

bool USplineComponent::ConvertSplineToPolyline_InTimeRange(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, float StartTimeAlongSpline, float EndTimeAlongSpline, bool bUseConstantVelocity, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline, bool bAllowWrappingIfClosed) const
{
	if (GetNumberOfSplinePoints() == 0)
	{
		return false;
	}

	// Helper to convert times to distances, so we can call the distance-based version of this function
	auto TimeToDistance = [this, bUseConstantVelocity, bAllowWrappingIfClosed](float Time) -> float
	{
		float TimeFrac = Time / Duration; // fraction of spline travelled
		if (bUseConstantVelocity)
		{
			return TimeFrac * GetSplineLength();
		}
		else
		{
			const int32 NumPoints = GetNumberOfSplinePoints();
			const int32 NumSegments = bClosedLoop ? NumPoints : NumPoints - 1;
			// Note: 'InputKey' values correspond to the spline in parameter space, in the range of 0 to NumSegments
			float InputKey = TimeFrac * NumSegments;
			if (bClosedLoop && bAllowWrappingIfClosed)
			{
				// Note the GetDistanceAlongSplineAtSplineInputKey() requires values in the 0-NumSegments range
				// So we wrap (modulus) into that range, find the distance, and then translate back to the original un-wrapped range.
				float DistanceAtStartOfLoop = FMath::Floor(TimeFrac) * GetSplineLength();
				float InRangeInputKey = FMath::Fmod(InputKey, NumSegments);
				if (InRangeInputKey < 0)
				{
					InRangeInputKey += NumSegments;
				}
				float DistanceWrapped = GetDistanceAlongSplineAtSplineInputKey(InRangeInputKey);
				return DistanceWrapped + DistanceAtStartOfLoop;
			}
			else
			{
				// If wrapping is not allowed, clamp to the valid range
				float ClampedInputKey = FMath::Clamp(InputKey, 0, NumSegments);
				return GetDistanceAlongSplineAtSplineInputKey(InputKey);
			}
		}
	};

	return ConvertSplineToPolyline_InDistanceRange(CoordinateSpace, MaxSquareDistanceFromSpline, TimeToDistance(StartTimeAlongSpline), TimeToDistance(EndTimeAlongSpline), OutPoints, OutDistancesAlongSpline, bAllowWrappingIfClosed);
}

template<class T>
T GetPropertyValueAtSplinePoint(const USplineMetadata* Metadata, int32 Index, FName PropertyName)
{
	if (Metadata)
	{
		if (FProperty* Property = Metadata->GetClass()->FindPropertyByName(PropertyName))
		{
			const FInterpCurve<T>* Curve = Property->ContainerPtrToValuePtr<FInterpCurve<T>>(Metadata);
			const TArray<FInterpCurvePoint<T>>& Points = Curve->Points;
			const int32 NumPoints = Points.Num();
			if (NumPoints > 0)
			{
				const int32 ClampedIndex = FMath::Clamp(Index, 0, NumPoints - 1);
				return Points[ClampedIndex].OutVal;
			}
		}
	}

	return T();
}

float USplineComponent::GetFloatPropertyAtSplinePoint(int32 Index, FName PropertyName) const
{
	return GetPropertyValueAtSplinePoint<float>(GetSplinePointsMetadata(), Index, PropertyName);
}

FVector USplineComponent::GetVectorPropertyAtSplinePoint(int32 Index, FName PropertyName) const
{
	return GetPropertyValueAtSplinePoint<FVector>(GetSplinePointsMetadata(), Index, PropertyName);
}

#if UE_ENABLE_DEBUG_DRAWING
void USplineComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	if (LineMaterial) { OutMaterials.Add(LineMaterial.Get()); }
	if (PointMaterial) { OutMaterials.Add(PointMaterial.Get()); }
}

FPrimitiveSceneProxy* USplineComponent::CreateSceneProxy()
{
	class FSplinePDISceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FSplinePDISceneProxy(const USplineComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
			, bDrawDebug(InComponent->bDrawDebug)
			, SplineInfo(InComponent->GetSplinePointsPosition())
#if WITH_EDITORONLY_DATA
			, LineColor(InComponent->EditorUnselectedSplineSegmentColor)
#else
			, LineColor(FLinearColor::White)
#endif
		{}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_SplineSceneProxy_GetDynamicMeshElements);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

					const FMatrix& LocalToWorld = GetLocalToWorld();

					// Taking into account the min and maximum drawing distance
					const float DistanceSqr = (View->ViewMatrices.GetViewOrigin() - LocalToWorld.GetOrigin()).SizeSquared();
					if (DistanceSqr < FMath::Square(GetMinDrawDistance()) || DistanceSqr > FMath::Square(GetMaxDrawDistance()))
					{
						continue;
					}

					USplineComponent::Draw(PDI, View, SplineInfo, LocalToWorld, LineColor, SDPG_World);
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = bDrawDebug && IsShown(View) && View->Family->EngineShowFlags.Splines;
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}

		virtual uint32 GetMemoryFootprint(void) const override { return sizeof *this + GetAllocatedSize(); }
		uint32 GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	private:
		bool bDrawDebug;
		FInterpCurveVector SplineInfo;
		FLinearColor LineColor;
	};

	class FSplineMeshSceneProxy final : public FPrimitiveSceneProxy
	{
		/** Inspired by FLineMeshBatchData and FPointMeshBatchData. */
		struct FMeshBatchData
		{
			FMaterialRenderProxy* MaterialProxy = nullptr;
			int32 StartIndex = 0;
			int32 NumPrimitives = 0;
			int32 MinVertexIndex = 0;
			int32 MaxVertexIndex = 0;
		};
		
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FSplineMeshSceneProxy(const USplineComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
			, bDrawDebug(InComponent->bDrawDebug)
			, SplineInfo(InComponent->GetSplinePointsPosition())
#if WITH_EDITORONLY_DATA
			, LineColor(InComponent->EditorUnselectedSplineSegmentColor)
#else
			, LineColor(FLinearColor::White)
#endif
			, LineVertexFactory(GetScene().GetFeatureLevel(), "FSplineProxyLineVertexFactory")
			, PointVertexFactory(GetScene().GetFeatureLevel(), "FSplineProxyPointVertexFactory")
		{
			CSV_SCOPED_TIMING_STAT_GLOBAL(FSplineSceneProxy_FSplineSceneProxy)
			
			constexpr int32 NumStepsPerSegment = 21;
			
			const int32 NumPoints = SplineInfo.Points.Num();
			const int32 NumSegments = FMath::Max(SplineInfo.bIsLooped ? NumPoints : NumPoints - 1, 0);
			const int32 NumLines = NumSegments * NumStepsPerSegment;
			
			const int32 NumLineVertices = NumLines * 4;
			const int32 NumLineIndices = NumLines * 6;
			const int32 NumPointVertices = NumPoints * 4;
			const int32 NumPointIndices = NumPoints * 6;
			constexpr int32 NumTextureCoordinates = 1;

			LineVertexBuffers.PositionVertexBuffer.Init(NumLineVertices);
			LineVertexBuffers.StaticMeshVertexBuffer.Init(NumLineVertices, NumTextureCoordinates);
			LineVertexBuffers.ColorVertexBuffer.Init(NumLineVertices);
			LineIndexBuffer.Indices.SetNumUninitialized(NumLineIndices);
			int32 LineVertexBufferIndex = 0;
			int32 LineIndexBufferIndex = 0;

			LineBatchData.MinVertexIndex = LineVertexBufferIndex;
			LineBatchData.MaxVertexIndex = LineVertexBufferIndex + NumLineVertices - 1;
			LineBatchData.StartIndex = LineIndexBufferIndex;
			LineBatchData.NumPrimitives = NumLines * 2;
			LineBatchData.MaterialProxy = InComponent->LineMaterialLifetimePtr->GetRenderProxy();

			PointVertexBuffers.PositionVertexBuffer.Init(NumPointVertices);
			PointVertexBuffers.StaticMeshVertexBuffer.Init(NumPointVertices, NumTextureCoordinates);
			PointVertexBuffers.ColorVertexBuffer.Init(NumPointVertices);
			PointIndexBuffer.Indices.SetNumUninitialized(NumPointIndices);
			int32 PointVertexBufferIndex = 0;
			int32 PointIndexBufferIndex = 0;

			PointBatchData.MinVertexIndex = PointVertexBufferIndex;
			PointBatchData.MaxVertexIndex = PointVertexBufferIndex + NumPointVertices - 1;
			PointBatchData.StartIndex = PointIndexBufferIndex;
			PointBatchData.NumPrimitives = NumPoints * 2;
			PointBatchData.MaterialProxy = InComponent->PointMaterialLifetimePtr->GetRenderProxy();
			
			auto AppendLineToBuffers = [this, &LineVertexBufferIndex, &LineIndexBufferIndex](FVector Start, FVector End, FColor Color, float Thickness, float DepthBias = 0.f) -> void
			{
				const FVector LineDirection = (End - Start).GetSafeNormal();
				const FVector2f UV(Thickness, DepthBias);

				LineVertexBuffers.PositionVertexBuffer.VertexPosition(LineVertexBufferIndex + 0) = static_cast<FVector3f>(Start);
				LineVertexBuffers.PositionVertexBuffer.VertexPosition(LineVertexBufferIndex + 1) = static_cast<FVector3f>(End);
				LineVertexBuffers.PositionVertexBuffer.VertexPosition(LineVertexBufferIndex + 2) = static_cast<FVector3f>(End);
				LineVertexBuffers.PositionVertexBuffer.VertexPosition(LineVertexBufferIndex + 3) = static_cast<FVector3f>(Start);

				// On each end of the line, we store the line direction pointing in opposite directions so that the verts
				// are moved in opposite directions to make the rectangle
				LineVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(LineVertexBufferIndex + 0, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(-LineDirection));
				LineVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(LineVertexBufferIndex + 1, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(-LineDirection));
				LineVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(LineVertexBufferIndex + 2, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(LineDirection));
				LineVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(LineVertexBufferIndex + 3, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(LineDirection));

				LineVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(LineVertexBufferIndex + 0, 0, UV);
				LineVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(LineVertexBufferIndex + 1, 0, UV);
				LineVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(LineVertexBufferIndex + 2, 0, UV);
				LineVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(LineVertexBufferIndex + 3, 0, UV);

				// The color stored in the vertices actually gets interpreted as a linear color by the material,
				// whereas it is more convenient for the user of the LineSet to specify colors as sRGB. So we actually
				// have to convert it back to linear. The ToFColor(false) call just scales back into 0-255 space.
				const FColor ConvertedColor = FLinearColor::FromSRGBColor(Color).ToFColor(false);
				LineVertexBuffers.ColorVertexBuffer.VertexColor(LineVertexBufferIndex + 0) = ConvertedColor;
				LineVertexBuffers.ColorVertexBuffer.VertexColor(LineVertexBufferIndex + 1) = ConvertedColor;
				LineVertexBuffers.ColorVertexBuffer.VertexColor(LineVertexBufferIndex + 2) = ConvertedColor;
				LineVertexBuffers.ColorVertexBuffer.VertexColor(LineVertexBufferIndex + 3) = ConvertedColor;

				LineIndexBuffer.Indices[LineIndexBufferIndex + 0] = LineVertexBufferIndex + 0;
				LineIndexBuffer.Indices[LineIndexBufferIndex + 1] = LineVertexBufferIndex + 1;
				LineIndexBuffer.Indices[LineIndexBufferIndex + 2] = LineVertexBufferIndex + 2;
				LineIndexBuffer.Indices[LineIndexBufferIndex + 3] = LineVertexBufferIndex + 2;
				LineIndexBuffer.Indices[LineIndexBufferIndex + 4] = LineVertexBufferIndex + 3;
				LineIndexBuffer.Indices[LineIndexBufferIndex + 5] = LineVertexBufferIndex + 0;

				LineVertexBufferIndex += 4;
				LineIndexBufferIndex += 6;
			};
			
			auto AppendPointToBuffers = [this, &PointVertexBufferIndex, &PointIndexBufferIndex](FVector Position, FColor Color, float Size, float DepthBias = 0.f) -> void
			{
				const FVector2f UV(Size, DepthBias);

				PointVertexBuffers.PositionVertexBuffer.VertexPosition(PointVertexBufferIndex + 0) = static_cast<FVector3f>(Position);
				PointVertexBuffers.PositionVertexBuffer.VertexPosition(PointVertexBufferIndex + 1) = static_cast<FVector3f>(Position);
				PointVertexBuffers.PositionVertexBuffer.VertexPosition(PointVertexBufferIndex + 2) = static_cast<FVector3f>(Position);
				PointVertexBuffers.PositionVertexBuffer.VertexPosition(PointVertexBufferIndex + 3) = static_cast<FVector3f>(Position);
				
				PointVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(PointVertexBufferIndex + 0, 0, UV);
				PointVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(PointVertexBufferIndex + 1, 0, UV);
				PointVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(PointVertexBufferIndex + 2, 0, UV);
				PointVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(PointVertexBufferIndex + 3, 0, UV);
				
				// The color stored in the vertices actually gets interpreted as a linear color by the material,
				// whereas it is more convenient for the user of the LineSet to specify colors as sRGB. So we actually
				// have to convert it back to linear. The ToFColor(false) call just scales back into 0-255 space.
				const FColor ConvertedColor = FLinearColor::FromSRGBColor(Color).ToFColor(false);
				PointVertexBuffers.ColorVertexBuffer.VertexColor(PointVertexBufferIndex + 0) = ConvertedColor;
				PointVertexBuffers.ColorVertexBuffer.VertexColor(PointVertexBufferIndex + 1) = ConvertedColor;
				PointVertexBuffers.ColorVertexBuffer.VertexColor(PointVertexBufferIndex + 2) = ConvertedColor;
				PointVertexBuffers.ColorVertexBuffer.VertexColor(PointVertexBufferIndex + 3) = ConvertedColor;
				
				PointVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(PointVertexBufferIndex + 0, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f(1.0f, -1.0f, 0.0f));
				PointVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(PointVertexBufferIndex + 1, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f(1.0f, 1.0f, 0.0f));
				PointVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(PointVertexBufferIndex + 2, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f(-1.0f, 1.0f, 0.0f));
				PointVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(PointVertexBufferIndex + 3, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f(-1.0f, -1.0f, 0.0f));
					
				PointIndexBuffer.Indices[PointIndexBufferIndex + 0] = PointVertexBufferIndex + 0;
				PointIndexBuffer.Indices[PointIndexBufferIndex + 1] = PointVertexBufferIndex + 1;
				PointIndexBuffer.Indices[PointIndexBufferIndex + 2] = PointVertexBufferIndex + 2;
				PointIndexBuffer.Indices[PointIndexBufferIndex + 3] = PointVertexBufferIndex + 2;
				PointIndexBuffer.Indices[PointIndexBufferIndex + 4] = PointVertexBufferIndex + 3;
				PointIndexBuffer.Indices[PointIndexBufferIndex + 5] = PointVertexBufferIndex + 0;

				PointVertexBufferIndex += 4;
				PointIndexBufferIndex += 6;
			};
			
			for (int32 KeyIdx = 0; KeyIdx < NumSegments + 1 && NumPoints > 0; KeyIdx++)
			{
				const float InputKey = KeyIdx < SplineInfo.Points.Num() ? SplineInfo.Points[KeyIdx].InVal : SplineInfo.Points.Last().InVal + SplineInfo.LoopKeyOffset;
				const FVector NewKeyPos = SplineInfo.Eval(InputKey);

				// Draw the keypoint
				if (KeyIdx < NumPoints)
				{
					constexpr int32 GrabHandleSize = 6;
					AppendPointToBuffers(NewKeyPos, LineColor.ToFColorSRGB(), GrabHandleSize);
				}

				// If not the first keypoint, draw a line to the previous keypoint.
				if (KeyIdx > 0)
				{
					const FInterpCurvePoint<FVector> OldKey = SplineInfo.Points[KeyIdx - 1];
					FVector OldKeyPos = OldKey.OutVal;
					
					// This initial value of 1.4f is an experimentally determined value which gives the same effective segment thickness as the
					// previous PDI rendering when the level editor thickness adjustment setting was at its default value (0). PDI special cases
					// lines of 0 thickness while our approach does not, so if we are not artificially increasing it here we will get invisible
					// lines by default.
					float SegmentLineThickness = 1.4f;
#if WITH_EDITOR
					SegmentLineThickness += GetDefault<ULevelEditorViewportSettings>()->SplineLineThicknessAdjustment;
#endif
					
					// For constant interpolation - don't draw ticks - just draw dotted line.
					if (SplineInfo.Points[KeyIdx - 1].InterpMode == CIM_Constant)
					{
						FVector OldPos = OldKeyPos;
						
						const FVector SegmentVector = NewKeyPos - OldKeyPos;
						const float SegmentLength = SegmentVector.Length();
						const FVector SegmentDirection = SegmentVector.GetSafeNormal();
						const float StepLength = SegmentLength / NumStepsPerSegment;
						const FVector SegmentStep = SegmentDirection * StepLength;
						
						for (int32 StepIdx = 1; StepIdx <= NumStepsPerSegment; StepIdx++)
						{
							const FVector NewPos = OldPos + SegmentStep;

							// In order to get a dashed line (which was the old behavior), we subdivide the line in the same way that we draw normal segments
							// but we hide every other subdivision by giving it a thickness of 0. While the fact that we render more lines than before may seem
							// wasteful, we do it for 2 reasons:
							// 1) Consistent logic for subdivision makes preallocating the vertex buffers trivial
							// 2) We are rendering lines in a much more efficient way than before
							const bool bHideLine = SplineInfo.Points[KeyIdx - 1].InterpMode == CIM_Constant && StepIdx % 2 == 0;
							AppendLineToBuffers(OldPos, NewPos, LineColor.ToFColorSRGB(), bHideLine ? 0.f : SegmentLineThickness);

							OldPos = NewPos;
						}
					}
					else
					{
						UE::Geometry::FInterval1f SegmentParameterRange(OldKey.InVal, InputKey);
						
						FVector OldPos = OldKeyPos;

						for (int32 StepIdx = 1; StepIdx <= NumStepsPerSegment; StepIdx++)
						{
							const float Key = SegmentParameterRange.Interpolate(static_cast<float>(StepIdx) / NumStepsPerSegment);
							const FVector NewPos = SplineInfo.Eval(Key, FVector(0));
							
							AppendLineToBuffers(OldPos, NewPos, LineColor.ToFColorSRGB(), SegmentLineThickness);

							OldPos = NewPos;
						}
					}
				}
			}

			if (NumLines > 0)
			{
				ENQUEUE_RENDER_COMMAND(LineSetVertexBuffersInit)(
					[this](FRHICommandListImmediate& RHICmdList)
				{
					LineVertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
					LineVertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
					LineVertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

					FLocalVertexFactory::FDataType Data;
					LineVertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&LineVertexFactory, Data);
					LineVertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&LineVertexFactory, Data);
					LineVertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&LineVertexFactory, Data);
					LineVertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&LineVertexFactory, Data);
					LineVertexFactory.SetData(RHICmdList, Data);

					LineVertexFactory.InitResource(RHICmdList);
					LineIndexBuffer.InitResource(RHICmdList);
				});
			}

			if (NumPoints > 0)
			{
				ENQUEUE_RENDER_COMMAND(OverlayVertexBuffersInit)(
					[this](FRHICommandListImmediate& RHICmdList)
				{
					PointVertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
					PointVertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
					PointVertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

					FLocalVertexFactory::FDataType Data;
					PointVertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&PointVertexFactory, Data);
					PointVertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&PointVertexFactory, Data);
					PointVertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&PointVertexFactory, Data);
					PointVertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&PointVertexFactory, Data);
					PointVertexFactory.SetData(RHICmdList, Data);

					PointVertexFactory.InitResource(RHICmdList);
					PointIndexBuffer.InitResource(RHICmdList);
				});
			}
		}

		virtual ~FSplineMeshSceneProxy() override
		{
			if (LineVertexBuffers.PositionVertexBuffer.IsInitialized())
			{
				LineVertexBuffers.PositionVertexBuffer.ReleaseResource();
				LineVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
				LineVertexBuffers.ColorVertexBuffer.ReleaseResource();
				LineIndexBuffer.ReleaseResource();
				LineVertexFactory.ReleaseResource();
			}

			if (PointVertexBuffers.PositionVertexBuffer.IsInitialized())
			{
				PointVertexBuffers.PositionVertexBuffer.ReleaseResource();
				PointVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
				PointVertexBuffers.ColorVertexBuffer.ReleaseResource();
				PointIndexBuffer.ReleaseResource();
				PointVertexFactory.ReleaseResource();
			}
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_SplineSceneProxy_GetDynamicMeshElements);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					if (LineVertexBuffers.PositionVertexBuffer.IsInitialized())
					{
						FMeshBatch& Mesh = Collector.AllocateMesh();
						FMeshBatchElement& BatchElement = Mesh.Elements[0];
						BatchElement.IndexBuffer = &LineIndexBuffer;
						Mesh.bWireframe = false;
						Mesh.VertexFactory = &LineVertexFactory;
						Mesh.MaterialRenderProxy = LineBatchData.MaterialProxy;

						FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
						DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), false, false, AlwaysHasVelocity());
						BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

						BatchElement.FirstIndex = LineBatchData.StartIndex;
						BatchElement.NumPrimitives = LineBatchData.NumPrimitives;
						BatchElement.MinVertexIndex = LineBatchData.MinVertexIndex;
						BatchElement.MaxVertexIndex = LineBatchData.MaxVertexIndex;
						Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
						Mesh.Type = PT_TriangleList;
						Mesh.DepthPriorityGroup = SDPG_World;
						Mesh.bCanApplyViewModeOverrides = false;
						Collector.AddMesh(ViewIndex, Mesh);
					}

					if (PointVertexBuffers.PositionVertexBuffer.IsInitialized())
					{
						FMeshBatch& Mesh = Collector.AllocateMesh();
						FMeshBatchElement& BatchElement = Mesh.Elements[0];
						BatchElement.IndexBuffer = &PointIndexBuffer;
						Mesh.bWireframe = false;
						Mesh.VertexFactory = &PointVertexFactory;
						Mesh.MaterialRenderProxy = PointBatchData.MaterialProxy;

						FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
						DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), true, false, AlwaysHasVelocity());
						BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

						BatchElement.FirstIndex = PointBatchData.StartIndex;
						BatchElement.NumPrimitives = PointBatchData.NumPrimitives;
						BatchElement.MinVertexIndex = PointBatchData.MinVertexIndex;
						BatchElement.MaxVertexIndex = PointBatchData.MaxVertexIndex;
						Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
						Mesh.Type = PT_TriangleList;
						Mesh.DepthPriorityGroup = SDPG_World;
						Mesh.bCanApplyViewModeOverrides = false;
						Collector.AddMesh(ViewIndex, Mesh);
					}
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = bDrawDebug && !IsSelected() && IsShown(View) && View->Family->EngineShowFlags.Splines;
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			Result.bRenderInMainPass = ShouldRenderInMainPass();
			Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
			Result.bRenderCustomDepth = ShouldRenderCustomDepth();
			Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
			Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
			return Result;
		}

		virtual uint32 GetMemoryFootprint(void) const override { return sizeof *this + GetAllocatedSize(); }
		uint32 GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	private:
		bool bDrawDebug;
		FInterpCurveVector SplineInfo;
		FLinearColor LineColor;
		
		FMeshBatchData LineBatchData;
		FLocalVertexFactory LineVertexFactory;
		FStaticMeshVertexBuffers LineVertexBuffers;
		FDynamicMeshIndexBuffer32 LineIndexBuffer;
		
		FMeshBatchData PointBatchData;
		FLocalVertexFactory PointVertexFactory;
		FStaticMeshVertexBuffers PointVertexBuffers;
		FDynamicMeshIndexBuffer32 PointIndexBuffer;
	};

	if (!bDrawDebug)
	{
		return Super::CreateSceneProxy();
	}

	if (!LineMaterialLifetimePtr && LineMaterialLoadID == INDEX_NONE)
	{
		LineMaterialLoadID = LineMaterial.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateWeakLambda(this, [this](const FSoftObjectPath&, UObject* LoadedObject) -> void
		{
			if (UMaterialInterface* LoadedMaterial = Cast<UMaterialInterface>(LoadedObject))
			{
				LineMaterialLifetimePtr = LoadedMaterial;
				MarkRenderStateDirty();
			}

			// We do not revert LineMaterialLoadID to INDEX_NONE to prevent multiple async load attempts
		}));
	}

	if (!PointMaterialLifetimePtr && PointMaterialLoadID == INDEX_NONE)
	{
		PointMaterialLoadID = PointMaterial.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateWeakLambda(this, [this](const FSoftObjectPath&, UObject* LoadedObject) -> void
		{
			if (UMaterialInterface* LoadedMaterial = Cast<UMaterialInterface>(LoadedObject))
			{
				PointMaterialLifetimePtr = LoadedMaterial;
				MarkRenderStateDirty();
			}

			// We do not revert PointMaterialLoadID to INDEX_NONE to prevent multiple async load attempts
		}));
	}
	
	if (!LineMaterialLifetimePtr || !PointMaterialLifetimePtr)
	{
		return new FSplinePDISceneProxy(this);
	}
	else
	{
		return new FSplineMeshSceneProxy(this);
	}
}
#endif
	
#if WITH_EDITOR

void USplineComponent::PushSelectionToProxy()
{
	if (!IsComponentIndividuallySelected())
	{
		OnDeselectedInEditor.Broadcast(this);
	}
	Super::PushSelectionToProxy();
}
#endif

#if UE_ENABLE_DEBUG_DRAWING

void USplineComponent::Draw(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FInterpCurveVector& SplineInfo, const FMatrix& LocalToWorld, const FLinearColor& LineColor, uint8 DepthPriorityGroup)
{
	const int32 GrabHandleSize = 6;

	auto GetLineColor = [LineColor](int32 KeyIdx, int32 NumSegments, int32 NumSteps) -> FLinearColor
	{
#if 1
		return LineColor;
#else
		float Normalized = (KeyIdx + (NumSegments * NumSteps)) / static_cast<float>(NumSegments * NumSteps);
		return FLinearColor::MakeFromHSV8(static_cast<uint8>(Normalized * 255), 255, 255);
#endif
	};
	
	const int32 NumPoints = SplineInfo.Points.Num();
	const int32 NumSegments = SplineInfo.bIsLooped ? NumPoints : NumPoints - 1;
	for (int32 KeyIdx = 0; KeyIdx < NumSegments + 1 && NumPoints > 0; KeyIdx++)
	{
		const float InputKey = KeyIdx < SplineInfo.Points.Num() ? SplineInfo.Points[KeyIdx].InVal : SplineInfo.Points.Last().InVal + SplineInfo.LoopKeyOffset;
		const FVector NewKeyPos = LocalToWorld.TransformPosition(SplineInfo.Eval(InputKey));

		// Draw the keypoint
		if (KeyIdx < NumPoints)
		{
			PDI->DrawPoint(NewKeyPos, LineColor, GrabHandleSize, DepthPriorityGroup);
		}

		// If not the first keypoint, draw a line to the previous keypoint.
		if (KeyIdx > 0)
		{
			const FInterpCurvePoint<FVector>& OldKey = SplineInfo.Points[KeyIdx - 1];
			const FVector OldKeyPos = LocalToWorld.TransformPosition(OldKey.OutVal);
			
			// For constant interpolation - don't draw ticks - just draw dotted line.
			if (OldKey.InterpMode == CIM_Constant)
			{
				// Calculate dash length according to size on screen
				const float StartW = View->WorldToScreen(OldKeyPos).W;
				const float EndW = View->WorldToScreen(NewKeyPos).W;

				constexpr float WLimit = 10.0f;
				if (StartW > WLimit || EndW > WLimit)
				{
					constexpr float Scale = 0.03f;
					DrawDashedLine(PDI, OldKeyPos, NewKeyPos, LineColor, FMath::Max(StartW, EndW) * Scale, DepthPriorityGroup);
				}
			}
			else
			{
				UE::Geometry::FInterval1f SegmentParameterRange(OldKey.InVal, InputKey);
				
				// Find position on first keyframe.
				FVector OldPos = OldKeyPos;

				// Then draw a line for each substep.
				constexpr int32 NumSteps = 20;
#if WITH_EDITOR
				const float SegmentLineThickness = GetDefault<ULevelEditorViewportSettings>()->SplineLineThicknessAdjustment;
#endif

				for (int32 StepIdx = 1; StepIdx <= NumSteps; StepIdx++)
				{
					const float SampleKey = SegmentParameterRange.Interpolate(static_cast<float>(StepIdx) / NumSteps);
					const FVector NewPos = LocalToWorld.TransformPosition(SplineInfo.Eval(SampleKey, FVector(0)));
#if WITH_EDITOR
					PDI->DrawTranslucentLine(OldPos, NewPos, GetLineColor(KeyIdx * NumSteps + StepIdx, NumSegments, NumSteps), DepthPriorityGroup, SegmentLineThickness);
#else
					PDI->DrawTranslucentLine(OldPos, NewPos, LineColor, DepthPriorityGroup);
#endif
					OldPos = NewPos;
				}
			}
		}
	}
}
#endif

FBoxSphereBounds USplineComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const auto& InterpCurve = GetSplinePointsPosition();

#if SPLINE_FAST_BOUNDS_CALCULATION
	FBox BoundingBox(0);
	for (const auto& InterpPoint : InterpCurve.Points)
	{
		BoundingBox += InterpPoint.OutVal;
	}

	return FBoxSphereBounds(BoundingBox.TransformBy(LocalToWorld));
#else
	const int32 NumPoints = InterpCurve.Points.Num();
	const int32 NumSegments = InterpCurve.bIsLooped ? NumPoints : NumPoints - 1;

	FVector Min(WORLD_MAX);
	FVector Max(-WORLD_MAX);
	if (NumSegments > 0)
	{
		for (int32 Index = 0; Index < NumSegments; Index++)
		{
			const bool bLoopSegment = (Index == NumPoints - 1);
			const int32 NextIndex = bLoopSegment ? 0 : Index + 1;
			const FInterpCurvePoint<FVector>& ThisInterpPoint = InterpCurve.Points[Index];
			FInterpCurvePoint<FVector> NextInterpPoint = InterpCurve.Points[NextIndex];
			if (bLoopSegment)
			{
				NextInterpPoint.InVal = ThisInterpPoint.InVal + InterpCurve.LoopKeyOffset;
			}

			CurveVectorFindIntervalBounds(ThisInterpPoint, NextInterpPoint, Min, Max);
		}
	}
	else if (NumPoints == 1)
	{
		Min = Max = InterpCurve.Points[0].OutVal;
	}
	else
	{
		Min = FVector::ZeroVector;
		Max = FVector::ZeroVector;
	}

	return FBoxSphereBounds(FBox(Min, Max).TransformBy(LocalToWorld));
#endif
}

bool USplineComponent::GetIgnoreBoundsForEditorFocus() const
{
	// Cannot compute proper bounds when there's no point so don't participate to editor focus if that's the case : 
	return Super::GetIgnoreBoundsForEditorFocus() || GetNumberOfSplinePoints() == 0;
}

TStructOnScope<FActorComponentInstanceData> USplineComponent::GetComponentInstanceData() const
{
	if (ShouldUseSplineCurves())
	{
		TStructOnScope<FActorComponentInstanceData> InstanceData = MakeStructOnScope<FActorComponentInstanceData, FSplineInstanceData>(this);
		FSplineInstanceData* SplineInstanceData = InstanceData.Cast<FSplineInstanceData>();
	
		if (bSplineHasBeenEdited)
		{
			SplineInstanceData->SplineCurves = GetSplineCurves();
			SplineInstanceData->bClosedLoop = bClosedLoop;
		}
		SplineInstanceData->bSplineHasBeenEdited = bSplineHasBeenEdited;
		
		return InstanceData;
	}
	else
	{
		TStructOnScope<FActorComponentInstanceData> InstanceData;
		
		if (bSplineHasBeenEdited)
		{
			InstanceData = MakeStructOnScope<FActorComponentInstanceData, FSplineComponentInstanceData>(this, Spline);
			FSplineComponentInstanceData* SplineComponentInstanceData = InstanceData.Cast<FSplineComponentInstanceData>();
			
			SplineComponentInstanceData->bClosedLoop = bClosedLoop;
			SplineComponentInstanceData->bSplineHasBeenEdited = bSplineHasBeenEdited;
		}
		else
		{
			InstanceData = MakeStructOnScope<FActorComponentInstanceData, FSplineComponentInstanceData>(this);
			FSplineComponentInstanceData* SplineComponentInstanceData = InstanceData.Cast<FSplineComponentInstanceData>();
			SplineComponentInstanceData->bSplineHasBeenEdited = bSplineHasBeenEdited;
		}

		return InstanceData;
	}
}

FSplineCurves USplineComponent::GetSplineCurves() const
{
	return ShouldUseSplineCurves()
		? WarninglessSplineCurves()
		: FSplineCurves::FromSplineInterface(Spline);
}

FSpline USplineComponent::GetSpline() const
{
	if (ShouldUseSplineCurves())
	{
		FSpline Intermediate;
		Intermediate = WarninglessSplineCurves();
		return Intermediate;
	}
	else
	{
		return Spline;
	}
}

void USplineComponent::SetSpline(const FSpline& InSpline)
{
	WarninglessSplineCurves() = FSplineCurves::FromSplineInterface(InSpline);
	Spline = InSpline;

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
	
	OnSplineChanged.Broadcast();
}

void USplineComponent::SetSpline(const FSplineCurves& InSplineCurves)
{
	WarninglessSplineCurves() = InSplineCurves;

	// If Spline is already tangent backed or SplineComponent.UseSplineCurves is false, we can copy assign to the FSpline.
	// This will force it to instantiate an FTangentSpline and populate it with the data from InSplineCurves.
	if (Spline.GetCurrentImplementation() == UE::Spline::ESplineType::Tangent || !UE::SplineComponent::GUseSplineCurves)
	{
		Spline = InSplineCurves;
	}

	if (UE::SplineComponent::GValidateOnChange)
	{
		Validate();
	}
	
	OnSplineChanged.Broadcast();
}

int32 USplineComponent::GetVersion() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Version;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetVersion();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0;
	}
}

const FInterpCurveVector& USplineComponent::GetSplinePointsPosition() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetSplinePointsPosition();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return DummyPointPosition;
	}
}

const FInterpCurveQuat& USplineComponent::GetSplinePointsRotation() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Rotation;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetSplinePointsRotation();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return DummyPointRotation;
	}
}

const FInterpCurveVector& USplineComponent::GetSplinePointsScale() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Scale;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetSplinePointsScale();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return DummyPointScale;
	}
}

TArray<ESplinePointType::Type> USplineComponent::GetEnabledSplinePointTypes() const
{
	return 
		{ 
			ESplinePointType::Linear,
			ESplinePointType::Curve,
			ESplinePointType::Constant,
			ESplinePointType::CurveClamped,
			ESplinePointType::CurveCustomTangent
		};
}

void USplineComponent::ApplyComponentInstanceData(FSplineInstanceData* SplineInstanceData, const bool bPostUCS)
{
	check(SplineInstanceData);

	if (bPostUCS)
	{
		if (bInputSplinePointsToConstructionScript)
		{
			// Don't reapply the saved state after the UCS has run if we are inputting the points to it.
			// This allows the UCS to work on the edited points and make its own changes.
			return;
		}
		else
		{
			bModifiedByConstructionScript = SplineInstanceData->SplineCurvesPreUCS != GetSplineCurves();

			// If we are restoring the saved state, unmark the SplineCurves property as 'modified'.
			// We don't want to consider that these changes have been made through the UCS.
			TArray<FProperty*> SplineProperties;
			for (const FName& Property : GetSplinePropertyNames())
			{
				SplineProperties.Add(FindFProperty<FProperty>(StaticClass(), Property));
			}
			
			RemoveUCSModifiedProperties(SplineProperties);
		}
	}
	else
	{
		SplineInstanceData->SplineCurvesPreUCS = GetSplineCurves();
	}

	if (SplineInstanceData->bSplineHasBeenEdited)
	{
		SetSpline(SplineInstanceData->SplineCurves);
		bClosedLoop = SplineInstanceData->bClosedLoop;
		bModifiedByConstructionScript = false;
	}

	bSplineHasBeenEdited = SplineInstanceData->bSplineHasBeenEdited;

	UpdateSpline();
}

void USplineComponent::ApplyComponentInstanceData(struct FSplineComponentInstanceData* SplineInstanceData, const bool bPostUCS)
{
	check(SplineInstanceData);

	if (bPostUCS)
	{
		if (bInputSplinePointsToConstructionScript)
		{
			// Don't reapply the saved state after the UCS has run if we are inputting the points to it.
			// This allows the UCS to work on the edited points and make its own changes.
			return;
		}
		else
		{
			bModifiedByConstructionScript = SplineInstanceData->SplinePreUCS != Spline;

			// If we are restoring the saved state, unmark the SplineCurves property as 'modified'.
			// We don't want to consider that these changes have been made through the UCS.
			TArray<FProperty*> SplineProperties;
			for (const FName& Property : GetSplinePropertyNames())
			{
				SplineProperties.Add(FindFProperty<FProperty>(StaticClass(), Property));
			}
			
			RemoveUCSModifiedProperties(SplineProperties);
		}
	}
	else
	{
		SplineInstanceData->SplinePreUCS = Spline;
	}

	if (SplineInstanceData->bSplineHasBeenEdited)
	{
		SetSpline(SplineInstanceData->Spline);
		bClosedLoop = SplineInstanceData->bClosedLoop;
		bModifiedByConstructionScript = false;
	}

	bSplineHasBeenEdited = SplineInstanceData->bSplineHasBeenEdited;

	UpdateSpline();
}

#if WITH_EDITOR
void USplineComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		const FName PropertyName(PropertyChangedEvent.Property->GetFName());
		
		static const FName ReparamStepsPerSegmentName = GET_MEMBER_NAME_CHECKED(USplineComponent, ReparamStepsPerSegment);
		static const FName StationaryEndpointsName = GET_MEMBER_NAME_CHECKED(USplineComponent, bStationaryEndpoints);
		static const FName DefaultUpVectorName = GET_MEMBER_NAME_CHECKED(USplineComponent, DefaultUpVector);
		static const FName ClosedLoopName = GET_MEMBER_NAME_CHECKED(USplineComponent, bClosedLoop);
		static const FName LoopPositionOverrideName = GET_MEMBER_NAME_CHECKED(USplineComponent, bLoopPositionOverride);
		static const FName LoopPositionName = GET_MEMBER_NAME_CHECKED(USplineComponent, LoopPosition);

		if (PropertyName == ReparamStepsPerSegmentName ||
			PropertyName == StationaryEndpointsName ||
			PropertyName == DefaultUpVectorName ||
			PropertyName == ClosedLoopName ||
			PropertyName == LoopPositionOverrideName ||
			PropertyName == LoopPositionName)
		{
			UpdateSpline();

			OnSplineChanged.Broadcast();
		}

		static const FName EditorUnselectedSplineSegmentColorName = GET_MEMBER_NAME_CHECKED(USplineComponent, EditorUnselectedSplineSegmentColor);
		static const FName EditorSelectedSplineSegmentColorName = GET_MEMBER_NAME_CHECKED(USplineComponent, EditorSelectedSplineSegmentColor);
		static const FName EditorTangentColorName = GET_MEMBER_NAME_CHECKED(USplineComponent, EditorTangentColor);
		static const FName AllowDiscontinuousSplineName = GET_MEMBER_NAME_CHECKED(USplineComponent, bAllowDiscontinuousSpline);
		static const FName AdjustTangentsOnSnapName = GET_MEMBER_NAME_CHECKED(USplineComponent, bAdjustTangentsOnSnap);
		static const FName ShouldVisualizeScaleName = GET_MEMBER_NAME_CHECKED(USplineComponent, bShouldVisualizeScale);
		static const FName ScaleVisualizationWidthName = GET_MEMBER_NAME_CHECKED(USplineComponent, ScaleVisualizationWidth);

		if (PropertyName == EditorUnselectedSplineSegmentColorName ||
			PropertyName == EditorSelectedSplineSegmentColorName ||
			PropertyName == EditorTangentColorName ||
			PropertyName == AllowDiscontinuousSplineName ||
			PropertyName == AdjustTangentsOnSnapName ||
			PropertyName == ShouldVisualizeScaleName ||
			PropertyName == ScaleVisualizationWidthName)
		{
			OnSplineDisplayChanged.Broadcast();
		}
		
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

void USplineComponent::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	// Notify listeners that the spline data may now be invalid.
	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		if (UE::SplineComponent::GValidateOnChange)
		{
			Validate();
		}
		
		OnSplineChanged.Broadcast();
	}
}
#endif

bool USplineComponent::ShouldUseSplineCurves() const
{
	return UE::SplineComponent::GUseSplineCurves || Spline.GetCurrentImplementation() != UE::Spline::ESplineType::Tangent;
}

UE::Geometry::FInterval1f USplineComponent::GetInputKeyRange() const
{
	UE::Geometry::FInterval1f Range;
	
	if (WarninglessSplineCurves().Position.Points.Num() > 0)
	{
		Range.Min = WarninglessSplineCurves().Position.Points[0].InVal;
		Range.Max = WarninglessSplineCurves().Position.Points.Last().InVal;
		
		// The valid range is extended by LoopKeyOffset for closed loops (this offset represents a virtual segment).
		if (WarninglessSplineCurves().Position.bIsLooped)
		{
			Range.Max += WarninglessSplineCurves().Position.LoopKeyOffset;
		}
	}
	
	return Range;
}

void FSplinePositionLinearApproximation::Build(const FSplineCurves& InCurves, TArray<FSplinePositionLinearApproximation>& OutPoints, float InDensity)
{
	OutPoints.Reset();

	const float SplineLength = InCurves.GetSplineLength();
	int32 NumLinearPoints = FMath::Max((int32)(SplineLength * InDensity), 2);

	for (int32 LinearPointIndex = 0; LinearPointIndex < NumLinearPoints; ++LinearPointIndex)
	{
		const float DistanceAlpha = (float)LinearPointIndex / (float)NumLinearPoints;
		const float SplineDistance = SplineLength * DistanceAlpha;
		const float Param = InCurves.ReparamTable.Eval(SplineDistance, 0.0f);
		OutPoints.Emplace(InCurves.Position.Eval(Param, FVector::ZeroVector), Param);
	}

	OutPoints.Emplace(InCurves.Position.Points.Last().OutVal, InCurves.ReparamTable.Points.Last().OutVal);
}