// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SplineMeshComponent.h"
#include "BodySetupEnums.h"
#include "Modules/ModuleManager.h"
#include "Engine/CollisionProfile.h"
#include "Materials/MaterialInterface.h"
#include "SceneView.h"
#include "StaticMeshResources.h"
#include "MeshDrawShaderBindings.h"
#include "SplineMeshSceneProxy.h"
#include "AI/NavigationSystemHelpers.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "Engine/StaticMeshSocket.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "MeshMaterialShader.h"
#include "PhysicsEngine/SphylElem.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "UObject/UnrealType.h"
#include "Materials/Material.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Engine/World.h"
#include "NaniteVertexFactory.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshComponentLODInfo.h"
#include "SplineMeshSceneProxyDesc.h"

#if WITH_EDITOR
#include "IHierarchicalLODUtilities.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "LandscapeSplineActor.h"
#include "LandscapeSplinesComponent.h"
#endif // WITH_EDITOR

namespace UE::SplineMesh
{
	float RealToFloatChecked(const double Value)
	{
		ensureMsgf(Value >= TNumericLimits<float>::Lowest() && Value <= TNumericLimits<float>::Max(), TEXT("Value %f exceeds float limits"), Value);
		return static_cast<float>(Value);
	}
}

int32 GNoRecreateSplineMeshProxy = 1;
static FAutoConsoleVariableRef CVarNoRecreateSplineMeshProxy(
	TEXT("r.SplineMesh.NoRecreateProxy"),
	GNoRecreateSplineMeshProxy,
	TEXT("Optimization. If true, spline mesh proxies will not be recreated every time they are changed. They are simply updated."));

int32 GSplineMeshRenderNanite = 1;
static FAutoConsoleVariableRef CVarSplineMeshRenderNanite(
	TEXT("r.SplineMesh.RenderNanite"),
	GSplineMeshRenderNanite,
	TEXT("When true, allows spline meshes to render as Nanite when enabled on the mesh (otherwise uses fallback mesh)."),
	FConsoleVariableDelegate::CreateLambda(
		[] (IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
		}
	)
);

bool USplineMeshComponent::ShouldRenderNaniteSplineMeshes()
{
	return NaniteSplineMeshesSupported() && GSplineMeshRenderNanite != 0;
}

void PackSplineMeshParams(const FSplineMeshShaderParams& Params, const TArrayView<FVector4f>& Output)
{
	auto PackF16 = [](float Value, uint32 Shift = 0) -> uint32
	{
		return uint32(FFloat16(Value).Encoded) << Shift;
	};
	auto PackSNorm16 = [](float Value, uint32 Shift = 0) -> uint32
	{
		const float N = FMath::Clamp(Value, -1.0f, 1.0f) * 0.5f + 0.5f;
		return uint32(N * 65535.0f) << Shift;
	};

	static_assert(SPLINE_MESH_PARAMS_FLOAT4_SIZE == 7, "If you changed the packed size of FSplineMeshShaderParams, this function needs to be updated");
	check(Output.Num() >= SPLINE_MESH_PARAMS_FLOAT4_SIZE);

	Output[0]	= FVector4f(Params.StartPos, Params.EndTangent.X);
	Output[1]	= FVector4f(Params.EndPos, Params.EndTangent.Y);
	Output[2]	= FVector4f(Params.StartTangent, Params.EndTangent.Z);
	Output[3]	= FVector4f(Params.StartOffset, Params.EndOffset);

	Output[4].X	= FMath::AsFloat(PackF16(Params.StartRoll) | PackF16(Params.EndRoll, 16u));
	Output[4].Y	= FMath::AsFloat(PackF16(Params.StartScale.X) | PackF16(Params.StartScale.Y, 16u));
	Output[4].Z	= FMath::AsFloat(PackF16(Params.EndScale.X) | PackF16(Params.EndScale.Y, 16u));
	Output[4].W	= FMath::AsFloat((Params.TextureCoord.X & 0xFFFFu) | (Params.TextureCoord.Y << 16u));

	Output[5].X	= Params.MeshZScale;
	Output[5].Y	= Params.MeshZOffset;
	Output[5].Z	= FMath::AsFloat(PackF16(Params.MeshDeformScaleMinMax.X) | PackF16(Params.MeshDeformScaleMinMax.Y, 16u));
	Output[5].W = FMath::AsFloat(PackF16(Params.SplineDistToTexelScale) | PackF16(Params.SplineDistToTexelOffset, 16u));

	Output[6].X = FMath::AsFloat(PackSNorm16(Params.SplineUpDir.X) | PackSNorm16(Params.SplineUpDir.Y, 16u));
	Output[6].Y = FMath::AsFloat(PackSNorm16(Params.SplineUpDir.Z) |
								 PackF16(FMath::Max(0.0f, Params.NaniteClusterBoundsScale), 16u) |
								 (Params.bSmoothInterpRollScale ? (1 << 31u) : 0u));

	const FQuat4f MeshRot = FQuat4f(FMatrix44f(Params.MeshDir, Params.MeshX, Params.MeshY, FVector3f::ZeroVector));
	Output[6].Z = FMath::AsFloat(PackSNorm16(MeshRot.X) | PackSNorm16(MeshRot.Y, 16u));
	Output[6].W = FMath::AsFloat(PackSNorm16(MeshRot.Z) | PackSNorm16(MeshRot.W, 16u));
}


FSplineMeshInstanceData::FSplineMeshInstanceData(const USplineMeshComponent* SourceComponent)
	: FStaticMeshComponentInstanceData(SourceComponent)
{
	StartPos = SourceComponent->SplineParams.StartPos;
	EndPos = SourceComponent->SplineParams.EndPos;
	StartTangent = SourceComponent->SplineParams.StartTangent;
	EndTangent = SourceComponent->SplineParams.EndTangent;
}

//////////////////////////////////////////////////////////////////////////
// FSplineMeshVertexFactoryShaderParameters


void FSplineMeshVertexFactoryShaderParameters::GetElementShaderBindings(
	const class FSceneInterface* Scene,
	const FSceneView* View,
	const class FMeshMaterialShader* Shader,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory,
	const FMeshBatchElement& BatchElement,
	class FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams
) const
{
	const EShaderPlatform ShaderPlatform = Scene ?
		Scene->GetShaderPlatform() :
		(View ? View->GetShaderPlatform() : GetFeatureLevelShaderPlatform(FeatureLevel));
	const bool bUseGPUScene = UseGPUScene(ShaderPlatform);
	const auto* LocalVertexFactory = static_cast<const FLocalVertexFactory*>(VertexFactory);
	
	if (BatchElement.bUserDataIsColorVertexBuffer)
	{
		const FColorVertexBuffer* OverrideColorVertexBuffer = (FColorVertexBuffer*)BatchElement.UserData;
		check(OverrideColorVertexBuffer);

		if (!LocalVertexFactory->SupportsManualVertexFetch(FeatureLevel))
		{
			LocalVertexFactory->GetColorOverrideStream(OverrideColorVertexBuffer, VertexStreams);
		}
	}
	
	if (bUseGPUScene)
	{
		FRHIUniformBuffer* VertexFactoryUniformBuffer = static_cast<FRHIUniformBuffer*>(BatchElement.VertexFactoryUserData);

		ShaderBindings.Add(Shader->GetUniformBufferParameter<FLocalVertexFactoryUniformShaderParameters>(), VertexFactoryUniformBuffer != nullptr ? VertexFactoryUniformBuffer : LocalVertexFactory->GetUniformBuffer());
	}

	if (BatchElement.LooseParametersUniformBuffer)
	{
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FSplineMeshVFPrimitiveShaderParameters>(), BatchElement.LooseParametersUniformBuffer);
	}

	}

//////////////////////////////////////////////////////////////////////////
// SplineMeshVertexFactory

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FSplineMeshVertexFactory, SF_Vertex     , FSplineMeshVertexFactoryShaderParameters);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FSplineMeshVertexFactory, SF_RayHitGroup, FSplineMeshVertexFactoryShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FSplineMeshVertexFactory, SF_Compute    , FSplineMeshVertexFactoryShaderParameters);
#endif

IMPLEMENT_VERTEX_FACTORY_TYPE(FSplineMeshVertexFactory, "/Engine/Private/LocalVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsStaticLighting
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPrecisePrevWorldPos
	| EVertexFactoryFlags::SupportsPositionOnly
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
	| EVertexFactoryFlags::SupportsManualVertexFetch
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
);

void InitSplineMeshVertexFactoryComponents(
	const FStaticMeshVertexBuffers& VertexBuffers, 
	const FSplineMeshVertexFactory* VertexFactory, 
	int32 LightMapCoordinateIndex, 
	bool bOverrideColorVertexBuffer,
	FLocalVertexFactory::FDataType& OutData)
{
	VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(VertexFactory, OutData);
	VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactory, OutData);
	VertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactory, OutData);
	VertexBuffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactory, OutData, LightMapCoordinateIndex);
	if (bOverrideColorVertexBuffer)
	{
		FColorVertexBuffer::BindDefaultColorVertexBuffer(VertexFactory, OutData, FColorVertexBuffer::NullBindStride::FColorSizeForComponentOverride);
	}
	else
	{
		VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(VertexFactory, OutData);
	}
}

//////////////////////////////////////////////////////////////////////////
// SplineMeshComponent

USplineMeshComponent::USplineMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Mobility = EComponentMobility::Static;

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	bAllowSplineEditingPerInstance = false;
	bSmoothInterpRollScale = false;
	bNeverNeedsCookedCollisionData = false;
	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::Yes;

	SplineUpDir.Z = 1.0f;

	// Default to useful length and scale
	SplineParams.StartTangent = FVector(100.f, 0.f, 0.f);
	SplineParams.StartScale = FVector2D(1.f, 1.f);

	SplineParams.EndPos = FVector(100.f, 0.f, 0.f);
	SplineParams.EndTangent = FVector(100.f, 0.f, 0.f);
	SplineParams.EndScale = FVector2D(1.f, 1.f);

	SplineBoundaryMin = 0;
	SplineBoundaryMax = 0;

	bMeshDirty = false;
}

void USplineMeshComponent::InitVertexFactory(int32 InLODIndex, FColorVertexBuffer* InOverrideColorVertexBuffer)
{
	FSplineMeshSceneProxyDesc::InitVertexFactory(GetStaticMesh(), GetWorld()->GetFeatureLevel(), InLODIndex, InOverrideColorVertexBuffer);
}

FVector USplineMeshComponent::GetStartPosition() const
{
	return SplineParams.StartPos;
}

void USplineMeshComponent::SetStartPosition(FVector StartPos, bool bUpdateMesh)
{
	SplineParams.StartPos = StartPos;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

FVector USplineMeshComponent::GetStartTangent() const
{
	return SplineParams.StartTangent;
}

void USplineMeshComponent::SetStartTangent(FVector StartTangent, bool bUpdateMesh)
{
	SplineParams.StartTangent = StartTangent;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

FVector USplineMeshComponent::GetEndPosition() const
{
	return SplineParams.EndPos;
}

void USplineMeshComponent::SetEndPosition(FVector EndPos, bool bUpdateMesh)
{
	SplineParams.EndPos = EndPos;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

FVector USplineMeshComponent::GetEndTangent() const
{
	return SplineParams.EndTangent;
}

void USplineMeshComponent::SetEndTangent(FVector EndTangent, bool bUpdateMesh)
{
	if (SplineParams.EndTangent == EndTangent)
	{
		return;
	}

	SplineParams.EndTangent = EndTangent;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

void USplineMeshComponent::SetStartAndEnd(FVector StartPos, FVector StartTangent, FVector EndPos, FVector EndTangent, bool bUpdateMesh)
{
	if (SplineParams.StartPos == StartPos && SplineParams.StartTangent == StartTangent && SplineParams.EndPos == EndPos && SplineParams.EndTangent == EndTangent)
	{
		return;
	}

	SplineParams.StartPos = StartPos;
	SplineParams.StartTangent = StartTangent;
	SplineParams.EndPos = EndPos;
	SetEndTangent(EndTangent, false);
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

FVector2D USplineMeshComponent::GetStartScale() const
{
	return SplineParams.StartScale;
}

void USplineMeshComponent::SetStartScale(FVector2D StartScale, bool bUpdateMesh)
{
	if (SplineParams.StartScale == StartScale)
	{
		return;
	}

	SplineParams.StartScale = StartScale;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

float USplineMeshComponent::GetStartRoll() const
{
	return SplineParams.StartRoll;
}

void USplineMeshComponent::SetStartRoll(float StartRoll, bool bUpdateMesh)
{
	SplineParams.StartRoll = StartRoll;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

void USplineMeshComponent::SetStartRollDegrees(float StartRollDegrees, bool bUpdateMesh)
{
	SetStartRoll(FMath::DegreesToRadians(StartRollDegrees), bUpdateMesh);
}

FVector2D USplineMeshComponent::GetStartOffset() const
{
	return SplineParams.StartOffset;
}

void USplineMeshComponent::SetStartOffset(FVector2D StartOffset, bool bUpdateMesh)
{
	SplineParams.StartOffset = StartOffset;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

FVector2D USplineMeshComponent::GetEndScale() const
{
	return SplineParams.EndScale;
}

void USplineMeshComponent::SetEndScale(FVector2D EndScale, bool bUpdateMesh)
{
	if (SplineParams.EndScale == EndScale)
	{
		return;
	}

	SplineParams.EndScale = EndScale;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

float USplineMeshComponent::GetEndRoll() const
{
	return SplineParams.EndRoll;
}

void USplineMeshComponent::SetEndRoll(float EndRoll, bool bUpdateMesh)
{
	SplineParams.EndRoll = EndRoll;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

void USplineMeshComponent::SetEndRollDegrees(float EndRollDegrees, bool bUpdateMesh)
{
	SetEndRoll(FMath::DegreesToRadians(EndRollDegrees), bUpdateMesh);
}

FVector2D USplineMeshComponent::GetEndOffset() const
{
	return SplineParams.EndOffset;
}

void USplineMeshComponent::SetEndOffset(FVector2D EndOffset, bool bUpdateMesh)
{
	SplineParams.EndOffset = EndOffset;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

ESplineMeshAxis::Type USplineMeshComponent::GetForwardAxis() const
{
	return ForwardAxis;
}

void USplineMeshComponent::SetForwardAxis(ESplineMeshAxis::Type InForwardAxis, bool bUpdateMesh)
{
	if (ForwardAxis == InForwardAxis)
	{
		return;
	}

	ForwardAxis = InForwardAxis;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

FVector USplineMeshComponent::GetSplineUpDir() const
{
	return SplineUpDir;
}

void USplineMeshComponent::SetSplineUpDir(const FVector& InSplineUpDir, bool bUpdateMesh)
{
	SplineUpDir = InSplineUpDir.GetSafeNormal();
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

float USplineMeshComponent::GetBoundaryMin() const
{
	return SplineBoundaryMin;
}

void USplineMeshComponent::SetBoundaryMin(float InBoundaryMin, bool bUpdateMesh)
{
	SplineBoundaryMin = InBoundaryMin;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

float USplineMeshComponent::GetBoundaryMax() const
{
	return SplineBoundaryMax;
}

void USplineMeshComponent::SetBoundaryMax(float InBoundaryMax, bool bUpdateMesh)
{
	SplineBoundaryMax = InBoundaryMax;
	bMeshDirty = true;
	if (bUpdateMesh)
	{
		UpdateRenderStateAndCollision();
	}
}

void USplineMeshComponent::SetbNeverNeedsCookedCollisionData(bool bInValue)
{
	bNeverNeedsCookedCollisionData = bInValue;
	if (BodySetup != nullptr)
	{
		BodySetup->bNeverNeedsCookedCollisionData = bInValue;
	}
}

void USplineMeshComponent::UpdateMesh()
{
	if (bMeshDirty)
	{
		UpdateRenderStateAndCollision();
	}
}

void USplineMeshComponent::UpdateMesh_Concurrent()
{
	if (bMeshDirty)
	{
		UpdateRenderStateAndCollision_Internal(true);
	}
}

FSplineMeshShaderParams USplineMeshComponent::CalculateShaderParams() const
{
	return FSplineMeshSceneProxyDesc(this).CalculateShaderParams();
}

void USplineMeshComponent::UpdateRenderStateAndCollision()
{
	UpdateRenderStateAndCollision_Internal(false);
}

void USplineMeshComponent::UpdateRenderStateAndCollision_Internal(bool bConcurrent)
{
	if (GNoRecreateSplineMeshProxy && bRenderStateCreated && SceneProxy)
	{
		if (bConcurrent)
		{
			SendRenderTransform_Concurrent();
		}
		else
		{
			MarkRenderTransformDirty();
		}

		ENQUEUE_RENDER_COMMAND(UpdateSplineParamsRTCommand)(
			[SceneProxy=SceneProxy, Params=CalculateShaderParams()](FRHICommandList&)
			{
				UpdateSplineMeshParams_RenderThread(SceneProxy, Params);
			}
		);
	}
	else
	{
		if (bConcurrent)
		{
			RecreateRenderState_Concurrent();
		}
		else
		{
			MarkRenderStateDirty();
		}
	}

	CachedMeshBodySetupGuid.Invalidate();
	RecreatePhysicsState();

	bMeshDirty = false;
}

void USplineMeshComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.UEVer() < VER_UE4_SPLINE_MESH_ORIENTATION)
	{
		ForwardAxis = ESplineMeshAxis::Z;
		SplineParams.StartRoll -= UE_HALF_PI;
		SplineParams.EndRoll -= UE_HALF_PI;

		double Temp = SplineParams.StartOffset.X;
		SplineParams.StartOffset.X = -SplineParams.StartOffset.Y;
		SplineParams.StartOffset.Y = Temp;
		Temp = SplineParams.EndOffset.X;
		SplineParams.EndOffset.X = -SplineParams.EndOffset.Y;
		SplineParams.EndOffset.Y = Temp;
	}

#if WITH_EDITOR
	if (BodySetup != nullptr)
	{
		BodySetup->SetFlags(RF_Transactional);
	}
#endif
}

#if WITH_EDITOR
bool USplineMeshComponent::IsRelevantForSplinePartitioning() const
{
	// Ignore editor only splines
	// Avoid using USplineMeshComponent::IsEditorOnly() as that would exclude all spline mesh components that are partitioned already
	if (Super::IsEditorOnly())
	{
		return false;
	}

	if (GetStaticMesh() == nullptr)
	{
		return false;
	}

	// Ignore editor splines
	if (GetStaticMesh() == GetDefault<ULandscapeSplinesComponent>()->SplineEditorMesh)
	{
		return false;
	}

	return true;
}

UHLODLayer* USplineMeshComponent::GetHLODLayerForSplinePartitioning() const
{
	if (HLODLayerOverrideForSplinePartitioning)
	{
		return HLODLayerOverrideForSplinePartitioning;
	}
	
	if (GetOwner())
	{
		return GetOwner()->GetHLODLayer();
	}

	return nullptr;
}

bool USplineMeshComponent::IsEditorOnly() const
{
	if (Super::IsEditorOnly())
	{
		return true;
	}

	// If landscape uses generated LandscapeSplineMeshesActors, SplineMeshComponents is removed from cooked build  
	const ALandscapeSplineActor* SplineActor = Cast<ALandscapeSplineActor>(GetOwner());
	if (SplineActor && SplineActor->HasGeneratedLandscapeSplineMeshesActors())
	{
		// Treat as editor only if the component was included in the spline partitioning
		return IsRelevantForSplinePartitioning();
	}

	return false;
}

bool USplineMeshComponent::Modify(bool bAlwaysMarkDirty)
{
	const bool bSavedToTransactionBuffer = Super::Modify(bAlwaysMarkDirty);

	if (BodySetup != nullptr)
	{
		// BodySetup shares the same package as its component.
		// Rely on the above call to Super::Modify(bAlwaysMarkDirty) to dirty the package if necessary.
		// Note that UActorComponent::Modify can force bAlwaysMarkDirty to false in some cases (like if the Actor is transient).
		BodySetup->Modify(false);
	}

	return bSavedToTransactionBuffer;
}
#endif

void USplineMeshComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	if (GetStaticMesh() == nullptr || GetStaticMesh()->GetRenderData() == nullptr)
	{
		return;
	}

	const FVertexFactoryType* VertexFactoryType = &FSplineMeshVertexFactory::StaticType;
	int32 LightMapCoordinateIndex = GetStaticMesh()->GetLightMapCoordinateIndex();

	auto SMC_GetElements = [LightMapCoordinateIndex, &LODData = this->LODData](const FStaticMeshLODResources& LODRenderData, int32 LODIndex, bool bSupportsManualVertexFetch, FVertexDeclarationElementList& Elements)
	{
		bool bOverrideColorVertexBuffer = LODIndex < LODData.Num() && LODData[LODIndex].OverrideVertexColors != nullptr;
		FLocalVertexFactory::FDataType Data;
		InitSplineMeshVertexFactoryComponents(LODRenderData.VertexBuffers, nullptr /*VertexFactory*/, LightMapCoordinateIndex, bOverrideColorVertexBuffer, Data);
		FLocalVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, EVertexInputStreamType::Default, bSupportsManualVertexFetch, Data, Elements);
	};

	FPSOPrecacheParams SplineMeshPSOParams = BasePrecachePSOParams;
	SplineMeshPSOParams.bSplineMesh = true;
	SplineMeshPSOParams.bReverseCulling ^= (SplineParams.StartScale.X < 0) ^ (SplineParams.StartScale.Y < 0);

	Nanite::FMaterialAudit NaniteMaterials{};
	if (ShouldCreateNaniteProxy(&NaniteMaterials))
	{
		CollectPSOPrecacheDataImpl(&FNaniteVertexFactory::StaticType, SplineMeshPSOParams, SMC_GetElements, OutParams);
	}
	else
	{
		CollectPSOPrecacheDataImpl(VertexFactoryType, SplineMeshPSOParams, SMC_GetElements, OutParams);
	}
}

FPrimitiveSceneProxy* USplineMeshComponent::CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	LLM_SCOPE(ELLMTag::StaticMesh);

    if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		return nullptr;
	}
	
	if (bCreateNanite && ShouldRenderNaniteSplineMeshes())
	{
		return ::new FNaniteSplineMeshSceneProxy(NaniteMaterials, this);
	}
	
	return ::new FSplineMeshSceneProxy(this);
}

FBoxSphereBounds USplineMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const UStaticMesh* Mesh = GetStaticMesh();
	if (Mesh == nullptr)
	{
		return FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);;
	}

	const FBox ComputedBounds = ComputeDistortedBounds(LocalToWorld, Mesh->GetBounds());
	return ComputedBounds;
}

void USplineMeshComponent::UpdateBounds()
{
	Super::UpdateBounds();

	CachedNavigationBounds = Bounds.GetBox();

	if (const UStaticMesh* Mesh = GetStaticMesh())
	{
		if (const UNavCollisionBase* NavCollision = Mesh->GetNavCollision())
		{
			// Match condition in DoCustomNavigableGeometryExport
			const FBox NavCollisionBounds = NavCollision->GetBounds();
			if (ensure(!NavCollision->IsDynamicObstacle())
				&& NavCollision->HasConvexGeometry()
				&& NavCollisionBounds.IsValid)
			{
				const FBoxSphereBounds NavCollisionBoxSphereBounds(NavCollisionBounds);
				CachedNavigationBounds = ComputeDistortedBounds(GetComponentTransform(), Mesh->GetBounds(), &NavCollisionBoxSphereBounds);
			}
		}
	}
}

float USplineMeshComponent::ComputeRatioAlongSpline(const float DistanceAlong) const 
{
	return FSplineMeshSceneProxyDesc(this).ComputeRatioAlongSpline(DistanceAlong);
}

void USplineMeshComponent::ComputeVisualMeshSplineTRange(float& MinT, float& MaxT) const
{
	return FSplineMeshSceneProxyDesc(this).ComputeVisualMeshSplineTRange(MinT, MaxT);
}

FBox USplineMeshComponent::ComputeDistortedBounds(const FTransform& InLocalToWorld, const FBoxSphereBounds& InMeshBounds, const FBoxSphereBounds* InBoundsToDistort) const
{
	return FSplineMeshSceneProxyDesc(this).ComputeDistortedBounds(InLocalToWorld, InMeshBounds, InBoundsToDistort);
}

bool USplineMeshComponent::SupportsDefaultCollision()
{
	// For Spline Mesh, if we have a Body Setup, we support the default collision, because the body setup is only initialized if the static mesh
	// has a body setup. But because the spline mesh body setup is different from the static mesh body setup (because of the deformation of the mesh)
	// the UStaticMeshComponent::SupportsDefaultCollision will always return false.
	return GetBodySetup() != nullptr;
}

FTransform USplineMeshComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	if (InSocketName != NAME_None)
	{
		UStaticMeshSocket const* const Socket = GetSocketByName(InSocketName);
		if (Socket)
		{
			FTransform SocketTransform(Socket->RelativeRotation, Socket->RelativeLocation * GetAxisMask(ForwardAxis), Socket->RelativeScale);
			SocketTransform = SocketTransform * CalcSliceTransform(UE::SplineMesh::RealToFloatChecked(GetAxisValueRef(Socket->RelativeLocation, ForwardAxis)));

			switch (TransformSpace)
			{
			case RTS_World:
			{
				return SocketTransform * GetComponentToWorld();
			}
			case RTS_Actor:
			{
				if (const AActor* Actor = GetOwner())
				{
					return (SocketTransform * GetComponentToWorld()).GetRelativeTransform(Actor->GetTransform());
				}
				break;
			}
			case RTS_Component:
			{
				return SocketTransform;
			}
			}
		}
	}

	return Super::GetSocketTransform(InSocketName, TransformSpace);
}

FTransform USplineMeshComponent::CalcSliceTransform(const float DistanceAlong) const
{
	return FSplineMeshSceneProxyDesc(this).CalcSliceTransform(DistanceAlong);
}

FTransform USplineMeshComponent::CalcSliceTransformAtSplineOffset(const float Alpha, const float MinT, const float MaxT) const
{
	return FSplineMeshSceneProxyDesc(this).CalcSliceTransformAtSplineOffset(Alpha, MinT, MaxT);
}

void USplineMeshComponent::PropagateSplineDeformationToMesh(FMeshDescription& InOutMeshDescription) const
{
	FStaticMeshAttributes Attributes(InOutMeshDescription);

	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();

	// Apply spline deformation for each vertex's tangents
	int32 WedgeIndex = 0;
	for (const FTriangleID TriangleID : InOutMeshDescription.Triangles().GetElementIDs())
	{
		for (int32 Corner = 0; Corner < 3; ++Corner, ++WedgeIndex)
		{
			const FVertexInstanceID VertexInstanceID = InOutMeshDescription.GetTriangleVertexInstance(TriangleID, Corner);
			const FVertexID VertexID = InOutMeshDescription.GetVertexInstanceVertex(VertexInstanceID);
			const float& AxisValue = USplineMeshComponent::GetAxisValueRef(VertexPositions[VertexID], ForwardAxis);
			FTransform SliceTransform = CalcSliceTransform(AxisValue);
			FVector TangentY = FVector::CrossProduct((FVector)VertexInstanceNormals[VertexInstanceID], (FVector)VertexInstanceTangents[VertexInstanceID]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceID];
			VertexInstanceTangents[VertexInstanceID] = (FVector3f)SliceTransform.TransformVector((FVector)VertexInstanceTangents[VertexInstanceID]);
			TangentY = SliceTransform.TransformVector(TangentY);
			VertexInstanceNormals[VertexInstanceID] = (FVector3f)SliceTransform.TransformVector((FVector)VertexInstanceNormals[VertexInstanceID]);
			VertexInstanceBinormalSigns[VertexInstanceID] = GetBasisDeterminantSign((FVector)VertexInstanceTangents[VertexInstanceID], TangentY, (FVector)VertexInstanceNormals[VertexInstanceID]);
		}
	}

	// Apply spline deformation for each vertex position
	for (const FVertexID VertexID : InOutMeshDescription.Vertices().GetElementIDs())
	{
		float& AxisValue = USplineMeshComponent::GetAxisValueRef(VertexPositions[VertexID], ForwardAxis);
		FTransform SliceTransform = CalcSliceTransform(AxisValue);
		AxisValue = 0.0f;
		VertexPositions[VertexID] = (FVector3f)SliceTransform.TransformPosition((FVector)VertexPositions[VertexID]);
	}
}

bool USplineMeshComponent::GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData)
{
	if (GetStaticMesh())
	{
		GetStaticMesh()->GetPhysicsTriMeshData(CollisionData, InUseAllTriData);

		FVector3f Mask = FVector3f(1, 1, 1);
		GetAxisValueRef(Mask, ForwardAxis) = 0;

		for (FVector3f& CollisionVert : CollisionData->Vertices)
		{
			CollisionVert = (FVector3f)CalcSliceTransform(GetAxisValueRef(CollisionVert, ForwardAxis)).TransformPosition(FVector(CollisionVert * Mask));
		}

		CollisionData->bDeformableMesh = true;

		return true;
	}

	return false;
}

bool USplineMeshComponent::ContainsPhysicsTriMeshData(bool InUseAllTriData) const
{
	if (GetStaticMesh())
	{
		return GetStaticMesh()->ContainsPhysicsTriMeshData(InUseAllTriData);
	}

	return false;
}

bool USplineMeshComponent::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const
{
	if (GetStaticMesh())
	{
		return GetStaticMesh()->GetTriMeshSizeEstimates(OutTriMeshEstimates, bInUseAllTriData);
	}

	return false;
}

FBox USplineMeshComponent::GetNavigationBounds() const
{
	return CachedNavigationBounds;
}

void USplineMeshComponent::GetMeshId(FString& OutMeshId)
{
	//call the const version
	return const_cast<const USplineMeshComponent*>(this)->GetMeshId(OutMeshId);
}

void USplineMeshComponent::GetMeshId(FString& OutMeshId) const
{
	// First get the base mesh id from the static mesh
	if (GetStaticMesh())
	{
		GetStaticMesh()->GetMeshId(OutMeshId);
	}

	// new method: Same guid as the base mesh but with a unique DDC-id based on the spline params.
	// This fixes the bug where running a blueprint construction script regenerates the guid and uses
	// a new DDC slot even if the mesh hasn't changed
	// If BodySetup is null that means we're *currently* duplicating one, and haven't transformed its data
	// to fit the spline yet, so just use the data from the base mesh by using a blank MeshId
	// It would be better if we could stop it building data in that case at all...

	if (BodySetup != nullptr && BodySetup->BodySetupGuid == CachedMeshBodySetupGuid)
	{
		FSplineMeshParams MutableSplineParams = SplineParams;
		FVector MutableSplineUpDir = SplineUpDir;
		float MutableSplineBoundaryMin = SplineBoundaryMin;
		float MutableSplineBoundaryMax = SplineBoundaryMax;
		TEnumAsByte<ESplineMeshAxis::Type> MutableForwardAxis = ForwardAxis;
		bool bMutableSmoothInterp = bSmoothInterpRollScale; 

		TArray<uint8> TempBytes;
		TempBytes.Reserve(256);

		FMemoryWriter Ar(TempBytes);
		Ar << MutableSplineParams.StartPos;
		Ar << MutableSplineParams.StartTangent;
		Ar << MutableSplineParams.StartScale;
		Ar << MutableSplineParams.StartRoll;
		Ar << MutableSplineParams.StartOffset;
		Ar << MutableSplineParams.EndPos;
		Ar << MutableSplineParams.EndTangent;
		Ar << MutableSplineParams.EndScale;
		Ar << MutableSplineParams.EndRoll;
		Ar << MutableSplineParams.EndOffset;
		Ar << MutableSplineUpDir;
		Ar << bMutableSmoothInterp;
		Ar << MutableForwardAxis;
		Ar << MutableSplineBoundaryMin;
		Ar << MutableSplineBoundaryMax;

		// Now convert the raw bytes to a string.
		const uint8* SettingsAsBytes = TempBytes.GetData();
		OutMeshId.Reserve(OutMeshId.Len() + TempBytes.Num() + 1);
		for (int32 ByteIndex = 0; ByteIndex < TempBytes.Num(); ++ByteIndex)
		{
			ByteToHex(SettingsAsBytes[ByteIndex], OutMeshId);
		}
	}
}

void USplineMeshComponent::OnCreatePhysicsState()
{
#if WITH_EDITOR
	if (IsVisualizationComponent())
	{
		return Super::OnCreatePhysicsState();
	}
#endif

	// With editor code we can recreate the collision if the mesh changes
	const FGuid MeshBodySetupGuid = (GetStaticMesh() != nullptr ? GetStaticMesh()->GetBodySetup()->BodySetupGuid : FGuid());
	if (CachedMeshBodySetupGuid != MeshBodySetupGuid)
	{
		RecreateCollision();
	}

	return Super::OnCreatePhysicsState();
}

UBodySetup* USplineMeshComponent::GetBodySetup()
{
	//call the const version
	return const_cast<const USplineMeshComponent*>(this)->GetBodySetup();
}

UBodySetup* USplineMeshComponent::GetBodySetup() const
{
	// Don't return a body setup that has no collision, it means we are interactively moving the spline and don't want to build collision.
	// Instead we explicitly build collision with USplineMeshComponent::RecreateCollision()
	if (BodySetup != nullptr && (BodySetup->TriMeshGeometries.Num() || BodySetup->AggGeom.GetElementCount() > 0))
	{
		return BodySetup;
	}

	return nullptr;
}

bool USplineMeshComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	// the NavCollision is supposed to be faster than exporting the regular collision,
	// but I'm not sure that's true here, as the regular collision is pre-distorted to the spline

	if (GetStaticMesh() != nullptr && GetStaticMesh()->GetNavCollision() != nullptr)
	{
		const UNavCollisionBase* NavCollision = GetStaticMesh()->GetNavCollision();

		if (ensure(!NavCollision->IsDynamicObstacle()))
		{
			if (NavCollision->HasConvexGeometry())
			{
				FVector Mask = FVector(1, 1, 1);
				GetAxisValueRef(Mask, ForwardAxis) = 0;

				TArray<FVector> VertexBuffer;
				VertexBuffer.Reserve(FMath::Max(NavCollision->GetConvexCollision().VertexBuffer.Num(), NavCollision->GetTriMeshCollision().VertexBuffer.Num()));

				for (int32 i = 0; i < NavCollision->GetConvexCollision().VertexBuffer.Num(); ++i)
				{
					FVector Vertex = NavCollision->GetConvexCollision().VertexBuffer[i];
					Vertex = CalcSliceTransform(UE::SplineMesh::RealToFloatChecked(GetAxisValueRef(Vertex, ForwardAxis))).TransformPosition(Vertex * Mask);
					VertexBuffer.Add(Vertex);
				}
				GeomExport.ExportCustomMesh(VertexBuffer.GetData(), VertexBuffer.Num(),
					NavCollision->GetConvexCollision().IndexBuffer.GetData(), NavCollision->GetConvexCollision().IndexBuffer.Num(),
					GetComponentTransform());

				VertexBuffer.Reset();
				for (int32 i = 0; i < NavCollision->GetTriMeshCollision().VertexBuffer.Num(); ++i)
				{
					FVector Vertex = NavCollision->GetTriMeshCollision().VertexBuffer[i];
					Vertex = CalcSliceTransform(UE::SplineMesh::RealToFloatChecked(GetAxisValueRef(Vertex, ForwardAxis))).TransformPosition(Vertex * Mask);
					VertexBuffer.Add(Vertex);
				}
				GeomExport.ExportCustomMesh(VertexBuffer.GetData(), VertexBuffer.Num(),
					NavCollision->GetTriMeshCollision().IndexBuffer.GetData(), NavCollision->GetTriMeshCollision().IndexBuffer.Num(),
					GetComponentTransform());

				return false;
			}
		}
	}

	return true;
}

void USplineMeshComponent::DestroyBodySetup()
{
	if (BodySetup != nullptr)
	{
		BodySetup->MarkAsGarbage();
		BodySetup = nullptr;
#if WITH_EDITORONLY_DATA
		CachedMeshBodySetupGuid.Invalidate();
#endif
	}
}

void USplineMeshComponent::RecreateCollision()
{
	if (GetStaticMesh())
	{
		if (BodySetup == nullptr)
		{
			BodySetup = DuplicateObject<UBodySetup>(GetStaticMesh()->GetBodySetup(), this);
			BodySetup->SetFlags(RF_Transactional);
			BodySetup->InvalidatePhysicsData();
		}
		else
		{
			const bool bDirtyPackage = false;
			BodySetup->Modify(bDirtyPackage);
			BodySetup->InvalidatePhysicsData();
			BodySetup->CopyBodyPropertiesFrom(GetStaticMesh()->GetBodySetup());
			BodySetup->CollisionTraceFlag = GetStaticMesh()->GetBodySetup()->CollisionTraceFlag;
		}
		BodySetup->BodySetupGuid = GetStaticMesh()->GetBodySetup()->BodySetupGuid;
		CachedMeshBodySetupGuid = GetStaticMesh()->GetBodySetup()->BodySetupGuid;

		BodySetup->bNeverNeedsCookedCollisionData = bNeverNeedsCookedCollisionData;

		if (BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple)
		{
			BodySetup->AggGeom.EmptyElements();
		}
		else
		{
			FVector Mask = FVector(1, 1, 1);
			GetAxisValueRef(Mask, ForwardAxis) = 0;

			// distortion of a sphere can't be done nicely, so we just transform the origin and size
			for (FKSphereElem& SphereElem : BodySetup->AggGeom.SphereElems)
			{
				const float Z = UE::SplineMesh::RealToFloatChecked(GetAxisValueRef(SphereElem.Center, ForwardAxis));
				FTransform SliceTransform = CalcSliceTransform(Z);
				SphereElem.Center *= Mask;

				SphereElem.Radius *= SliceTransform.GetMaximumAxisScale();
				SphereElem.Center = SliceTransform.TransformPosition(SphereElem.Center);
			}

			// distortion of a sphyl can't be done nicely, so we just transform the origin and size
			for (FKSphylElem& SphylElem : BodySetup->AggGeom.SphylElems)
			{
				const float Z = UE::SplineMesh::RealToFloatChecked(GetAxisValueRef(SphylElem.Center, ForwardAxis));
				FTransform SliceTransform = CalcSliceTransform(Z);
				SphylElem.Center *= Mask;

				FTransform TM = SphylElem.GetTransform();
				SphylElem.Length = UE::SplineMesh::RealToFloatChecked((TM * SliceTransform).TransformVector(FVector(0, 0, SphylElem.Length)).Size());
				SphylElem.Radius *= SliceTransform.GetMaximumAxisScale();

				SphylElem.SetTransform(TM * SliceTransform);
			}

			// Convert boxes to convex hulls to better respect distortion
			for (FKBoxElem& BoxElem : BodySetup->AggGeom.BoxElems)
			{
				FKConvexElem& ConvexElem = BodySetup->AggGeom.ConvexElems.AddDefaulted_GetRef();

				const FVector Radii = FVector(BoxElem.X / 2, BoxElem.Y / 2, BoxElem.Z / 2).ComponentMax(FVector(1.0f));
				const FTransform ElementTM = BoxElem.GetTransform();
				ConvexElem.VertexData.Empty(8);
				ConvexElem.VertexData.Add(ElementTM.TransformPosition(Radii * FVector(-1, -1, -1)));
				ConvexElem.VertexData.Add(ElementTM.TransformPosition(Radii * FVector(-1, -1, 1)));
				ConvexElem.VertexData.Add(ElementTM.TransformPosition(Radii * FVector(-1, 1, -1)));
				ConvexElem.VertexData.Add(ElementTM.TransformPosition(Radii * FVector(-1, 1, 1)));
				ConvexElem.VertexData.Add(ElementTM.TransformPosition(Radii * FVector(1, -1, -1)));
				ConvexElem.VertexData.Add(ElementTM.TransformPosition(Radii * FVector(1, -1, 1)));
				ConvexElem.VertexData.Add(ElementTM.TransformPosition(Radii * FVector(1, 1, -1)));
				ConvexElem.VertexData.Add(ElementTM.TransformPosition(Radii * FVector(1, 1, 1)));

				ConvexElem.UpdateElemBox();
			}
			BodySetup->AggGeom.BoxElems.Empty();

			// transform the points of the convex hulls into spline space
			for (FKConvexElem& ConvexElem : BodySetup->AggGeom.ConvexElems)
			{
				FTransform TM = ConvexElem.GetTransform();
				for (FVector& Point : ConvexElem.VertexData)
				{
					// pretransform the point by its local transform so we are working in untransformed local space
					FVector TransformedPoint = TM.TransformPosition(Point);
					// apply the transform to spline space
					Point = CalcSliceTransform(UE::SplineMesh::RealToFloatChecked(GetAxisValueRef(TransformedPoint, ForwardAxis))).TransformPosition(TransformedPoint * Mask);
				}

				// Set the local transform as an identity as points have already been transformed
				ConvexElem.SetTransform(FTransform::Identity);
				ConvexElem.UpdateElemBox();
			}
		}

		BodySetup->CreatePhysicsMeshes();

		// After the BodySetup is created/updated, we need to make sure the default collision is respected
		// This function is called in the OnRegister, and at that point for Spline Meshes, BodySetup is null.
		UpdateCollisionFromStaticMesh();
	}
	else
	{
		DestroyBodySetup();
	}
}

TStructOnScope<FActorComponentInstanceData> USplineMeshComponent::GetComponentInstanceData() const
{
	TStructOnScope<FActorComponentInstanceData> InstanceData;
	if (bAllowSplineEditingPerInstance)
	{
		InstanceData = MakeStructOnScope<FActorComponentInstanceData, FSplineMeshInstanceData>(this);
	}
	else
	{
		InstanceData = Super::GetComponentInstanceData();
	}
	return InstanceData;
}

void USplineMeshComponent::ApplyComponentInstanceData(FSplineMeshInstanceData* SplineMeshInstanceData)
{
	if (SplineMeshInstanceData)
	{
		if (bAllowSplineEditingPerInstance)
		{
			SplineParams.StartPos = SplineMeshInstanceData->StartPos;
			SplineParams.EndPos = SplineMeshInstanceData->EndPos;
			SplineParams.StartTangent = SplineMeshInstanceData->StartTangent;
			SetEndTangent(SplineMeshInstanceData->EndTangent, false);
			UpdateRenderStateAndCollision();
		}
	}
}


#include "PhysicsEngine/SphereElem.h"
#include "StaticMeshLight.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineMeshComponent)

/** */
class FSplineStaticLightingMesh : public FStaticMeshStaticLightingMesh
{
public:

	FSplineStaticLightingMesh(const USplineMeshComponent* InPrimitive, int32 InLODIndex, const TArray<ULightComponent*>& InRelevantLights) :
		FStaticMeshStaticLightingMesh(InPrimitive, InLODIndex, InRelevantLights),
		SplineComponent(InPrimitive)
	{
	}

#if WITH_EDITOR
	virtual const struct FSplineMeshParams* GetSplineParameters() const
	{
		return &SplineComponent->SplineParams;
	}
#endif	//WITH_EDITOR

private:
	const USplineMeshComponent* SplineComponent;
};

FStaticMeshStaticLightingMesh* USplineMeshComponent::AllocateStaticLightingMesh(int32 LODIndex, const TArray<ULightComponent*>& InRelevantLights)
{
	return new FSplineStaticLightingMesh(this, LODIndex, InRelevantLights);
}


float USplineMeshComponent::GetTextureStreamingTransformScale() const
{
	FVector::FReal SplineDeformFactor = 1;

	if (GetStaticMesh())
	{
		// We do this by looking at the ratio between current bounds (including deformation) and undeformed (straight from staticmesh)
		constexpr float MinExtent = 1.0f;
		const FBoxSphereBounds UndeformedBounds = GetStaticMesh()->GetBounds().TransformBy(GetComponentTransform());
		if (UndeformedBounds.BoxExtent.X >= MinExtent)
		{
			SplineDeformFactor = FMath::Max(SplineDeformFactor, Bounds.BoxExtent.X / UndeformedBounds.BoxExtent.X);
		}
		if (UndeformedBounds.BoxExtent.Y >= MinExtent)
		{
			SplineDeformFactor = FMath::Max(SplineDeformFactor, Bounds.BoxExtent.Y / UndeformedBounds.BoxExtent.Y);
		}
		if (UndeformedBounds.BoxExtent.Z >= MinExtent)
		{
			SplineDeformFactor = FMath::Max(SplineDeformFactor, Bounds.BoxExtent.Z / UndeformedBounds.BoxExtent.Z);
		}
	}

	return UE::SplineMesh::RealToFloatChecked(SplineDeformFactor) * Super::GetTextureStreamingTransformScale();
}

#if WITH_EDITOR
void USplineMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FProperty* MemberPropertyThatChanged = PropertyChangedEvent.MemberProperty;
	const bool bIsSplineParamsChange = MemberPropertyThatChanged && MemberPropertyThatChanged->GetNameCPP() == TEXT("SplineParams");
	if (bIsSplineParamsChange)
	{
		SetEndTangent(SplineParams.EndTangent, false);
	}

	UStaticMeshComponent::PostEditChangeProperty(PropertyChangedEvent);

	// If the spline params were changed the actual geometry is, so flag the owning HLOD cluster as dirty
	if (bIsSplineParamsChange)
	{
		IHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<IHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
		IHierarchicalLODUtilities* Utilities = Module.GetUtilities();
		Utilities->HandleActorModified(GetOwner());
	}

	if (MemberPropertyThatChanged && (MemberPropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(USplineMeshComponent, bNeverNeedsCookedCollisionData)))
	{
		// TODO [jonathan.bard] : this is currently needed because Setter doesn't correctly do its job in the details panel but eventually this could be removed : 
		SetbNeverNeedsCookedCollisionData(bNeverNeedsCookedCollisionData);
	}
}
#endif

