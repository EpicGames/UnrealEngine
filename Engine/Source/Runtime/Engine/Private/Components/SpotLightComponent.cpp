// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SpotLightComponent.h"
#include "RenderUtils.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"
#include "PointLightSceneProxy.h"
#include "SpotLightSceneProxy.h"
#include "UObject/UnrealType.h"
#include "SceneInterface.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpotLightComponent)

USpotLightComponent::USpotLightComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UTexture2D> StaticTexture(TEXT("/Engine/EditorResources/LightIcons/S_LightSpot"));
		static ConstructorHelpers::FObjectFinder<UTexture2D> DynamicTexture(TEXT("/Engine/EditorResources/LightIcons/S_LightSpotMove"));

		StaticEditorTexture = StaticTexture.Object;
		StaticEditorTextureScale = 0.5f;
		DynamicEditorTexture = DynamicTexture.Object;
		DynamicEditorTextureScale = 0.5f;
	}
#endif

	InnerConeAngle = 0.0f;
	OuterConeAngle = 44.0f;
}

FVector2f USpotLightComponent::GetClampedConeAngles(float InInnerConeAngle, float InOuterConeAngle)
{
	FVector2f ConeAngles;
	ConeAngles.X = FMath::Clamp(InInnerConeAngle, 0.0f, 88.9f) * (float)UE_PI / 180.0f;
	ConeAngles.Y = FMath::Clamp(InOuterConeAngle * (float)UE_PI / 180.0f, ConeAngles.X + 0.001f, 88.9f * (float)UE_PI / 180.0f + 0.001f);
	return ConeAngles;
}

FVector2f USpotLightComponent::GetClampedConeAngles() const
{
	return GetClampedConeAngles(InnerConeAngle, OuterConeAngle);
}

float USpotLightComponent::GetHalfConeAngle() const
{
	const FVector2f ClampedConeAngles = GetClampedConeAngles();
	return ClampedConeAngles.Y;
}

float USpotLightComponent::GetCosHalfConeAngle() const
{
	return FMath::Cos(GetHalfConeAngle());
}

void USpotLightComponent::SetInnerConeAngle(float NewInnerConeAngle)
{
	if (AreDynamicDataChangesAllowed(false)
		&& NewInnerConeAngle != InnerConeAngle)
	{
		InnerConeAngle = NewInnerConeAngle;
		MarkRenderStateDirty();
	}
}

void USpotLightComponent::SetOuterConeAngle(float NewOuterConeAngle)
{
	if (AreDynamicDataChangesAllowed(false)
		&& NewOuterConeAngle != OuterConeAngle)
	{
		OuterConeAngle = NewOuterConeAngle;
		MarkRenderStateDirty();
	}
}

float USpotLightComponent::ComputeLightBrightness() const
{
	float LightBrightness = ULightComponent::ComputeLightBrightness();

	if (bUseInverseSquaredFalloff)
	{
		if (IntensityUnits == ELightUnits::Candelas)
		{
			LightBrightness *= (100.f * 100.f); // Conversion from cm2 to m2
		}
		else if (IntensityUnits == ELightUnits::Nits)
		{
			// Capsule area (sphere+cylinder)
			float AreaInCm2 = (4 * UE_PI) * SourceRadius * (SourceRadius + 0.5f * SourceLength);
			LightBrightness *= AreaInCm2;
		}
		else if (IntensityUnits == ELightUnits::Lumens)
		{
			LightBrightness *= (100.f * 100.f / 2.f / UE_PI / (1.f - GetCosHalfConeAngle())); // Conversion from cm2 to m2 and cone remapping.
		}
		else if (IntensityUnits == ELightUnits::EV)
		{
			LightBrightness = EV100ToLuminance(LightBrightness) * (100.f * 100.f);
		}
		else
		{
			LightBrightness *= 16; // Legacy scale of 16
		}
	}
	return LightBrightness;
}

#if WITH_EDITOR
void USpotLightComponent::SetLightBrightness(float InBrightness)
{
	if (bUseInverseSquaredFalloff)
	{
		if (IntensityUnits == ELightUnits::Candelas)
		{
			ULightComponent::SetLightBrightness(InBrightness / (100.f * 100.f)); // Conversion from cm2 to m2
		}
		else if (IntensityUnits == ELightUnits::Nits)
		{
			// Capsule area (sphere+cylinder)
			float AreaInCm2 = (4 * UE_PI) * SourceRadius * (SourceRadius + 0.5f * SourceLength);
			ULightComponent::SetLightBrightness(InBrightness / AreaInCm2);
		}
		else if (IntensityUnits == ELightUnits::Lumens)
		{
			ULightComponent::SetLightBrightness(InBrightness / (100.f * 100.f / 2.f / UE_PI / (1.f - GetCosHalfConeAngle()))); // Conversion from cm2 to m2 and cone remapping
		}
		else if (IntensityUnits == ELightUnits::EV)
		{
			ULightComponent::SetLightBrightness(LuminanceToEV100(InBrightness / (100.f * 100.f)));
		}
		else
		{
			ULightComponent::SetLightBrightness(InBrightness / 16); // Legacy scale of 16
		}
	}
	else
	{
		ULightComponent::SetLightBrightness(InBrightness);
	}
}

FBox USpotLightComponent::GetStreamingBounds() const
{
	const FSphere BoundingSphere = GetBoundingSphere();
	return FBox(BoundingSphere);
}
#endif // WITH_EDITOR

static bool IsSpotLightSupported(const USpotLightComponent* InLight)
{
	if (GMaxRHIFeatureLevel <= ERHIFeatureLevel::ES3_1 && InLight->IsMovable())
	{
		if (!IsMobileDeferredShadingEnabled(GMaxRHIShaderPlatform))
		{
			// if project does not support dynamic point/spot lights on mobile do not add them to the renderer 
			return MobileForwardEnableLocalLights(GMaxRHIShaderPlatform);
		}
	}
	return true;
}

FLightSceneProxy* USpotLightComponent::CreateSceneProxy() const
{
	if (IsSpotLightSupported(this) && !IsPSOPrecaching())
	{
		return new FSpotLightSceneProxy(this);
	}
	return nullptr;
}

void USpotLightComponent::PrecachePSOs()
{
	if (IsSpotLightSupported(this))
	{
		Super::PrecachePSOs();
	}
}

FSphere USpotLightComponent::GetBoundingSphere() const
{
	FSphere::FReal ConeAngle = GetHalfConeAngle();
	FSphere::FReal CosConeAngle = FMath::Cos(ConeAngle);
	FSphere::FReal SinConeAngle = FMath::Sin(ConeAngle);
	return FMath::ComputeBoundingSphereForCone(GetComponentTransform().GetLocation(), GetDirection(), (FSphere::FReal)AttenuationRadius, CosConeAngle, SinConeAngle);
}

bool USpotLightComponent::AffectsBounds(const FBoxSphereBounds& InBounds) const
{
	if(!Super::AffectsBounds(InBounds))
	{
		return false;
	}

	const FVector2f ClampedConeAngles = GetClampedConeAngles();

	float	Sin = FMath::Sin(ClampedConeAngles.Y),
			Cos = FMath::Cos(ClampedConeAngles.Y);

	FVector	U = GetComponentLocation() - (InBounds.SphereRadius / Sin) * GetDirection(),
			D = InBounds.Origin - U;
	float	dsqr = D | D,
			E = GetDirection() | D;
	if(E > 0.0f && E * E >= dsqr * FMath::Square(Cos))
	{
		D = InBounds.Origin - GetComponentLocation();
		dsqr = D | D;
		E = -(GetDirection() | D);
		if(E > 0.0f && E * E >= dsqr * FMath::Square(Sin))
			return dsqr <= FMath::Square(InBounds.SphereRadius);
		else
			return true;
	}

	return false;
}

/**
* @return ELightComponentType for the light component class 
*/
ELightComponentType USpotLightComponent::GetLightType() const
{
	return LightType_Spot;
}

#if WITH_EDITOR

void USpotLightComponent::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == TEXT("InnerConeAngle"))
		{
			OuterConeAngle = FMath::Max(InnerConeAngle, OuterConeAngle);
		}
		else if (PropertyChangedEvent.Property->GetFName() == TEXT("OuterConeAngle"))
		{
			InnerConeAngle = FMath::Min(InnerConeAngle, OuterConeAngle);
		}
	}

	UPointLightComponent::PostEditChangeProperty(PropertyChangedEvent);
}

#endif	// WITH_EDITOR

void USpotLightComponent::BuildSceneProxyDesc(const USpotLightComponent& InLightComponent, FSpotLightSceneProxyDesc& OutProxyDesc)
{
	UPointLightComponent::BuildSceneProxyDesc(InLightComponent, OutProxyDesc);

	OutProxyDesc.InnerConeAngle = InLightComponent.InnerConeAngle;
	OutProxyDesc.OuterConeAngle = InLightComponent.OuterConeAngle;
}