// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/StereoLayerComponent.h"
#include "Engine/Engine.h"
#include "TextureResource.h"
#include "Engine/Texture.h"
#include "StereoRendering.h"
#include "StereoLayerAdditionalFlagsManager.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(StereoLayerComponent)
#if WITH_EDITOR
#include "DynamicMeshBuilder.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialRenderProxy.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#else
#include "StereoRendering.h"
#endif

#if WITH_EDITORONLY_DATA
void DrawStereoLayerQuad(
	FPrimitiveDrawInterface* PDI,
	const FVector& Center,
	const FVector& XAxis,
	const FVector& YAxis, 
	FColor Color,
	float Width,
	float Height,
	const FMaterialRenderProxy* MaterialRenderProxy,
	uint8 DepthPriority)
{
	FVector XOffset = XAxis * Width * 0.5f;
	FVector YOffset = YAxis * Height * 0.5f;

	// Calculate verts for a face lying in plane defined by the X and Y vectors
	FVector Positions[4] =
	{
		Center - XOffset - YOffset,
		Center + XOffset - YOffset,
		Center + XOffset + YOffset,
		Center - XOffset + YOffset
	};

	FVector2f UVs[4] =
	{
		FVector2f(0.0f,0.0f),
		FVector2f(0.0f,1.0f),
		FVector2f(1.0f,1.0f),
		FVector2f(1.0f,0.0f),
	};

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	int32 VertexIndices[4];
	for (int32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
	{
		VertexIndices[VertexIndex] = MeshBuilder.AddVertex(
			(FVector3f)Positions[VertexIndex],
			UVs[VertexIndex],
			FVector3f(1.0f, 0.0f, 0.0f),
			FVector3f(0.0f, 1.0f, 0.0f),
			FVector3f(0.0f, 0.0f, 1.0f),
			Color
		);
	}

	MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[2], VertexIndices[1]);
	MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[3], VertexIndices[2]);

	MeshBuilder.Draw(PDI, FMatrix::Identity, MaterialRenderProxy, DepthPriority, false);
}
#endif

UStereoLayerComponent::UStereoLayerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bLiveTexture(false)
	, bSupportsDepth(false)
	, bNoAlphaChannel(false)
	, Texture(nullptr)
	, LeftTexture(nullptr)
	, bQuadPreserveTextureRatio(false)
	, QuadSize(FVector2D(100.0f, 100.0f))
	, UVRect(FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f)))
	, StereoLayerType(SLT_FaceLocked)
	, Priority(0)
	, LayerId(IStereoLayers::FLayerDesc::INVALID_LAYER_ID)
	, bIsDirty(true)
	, bTextureNeedsUpdate(false)
	, LastTransform(FTransform::Identity)
	, bLastVisible(false)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	// The following somehow overrides subobjects read through serialization.
	//Shape = ObjectInitializer.CreateDefaultSubobject<UStereoLayerShapeQuad>(this, TEXT("Shape"));
}

void UStereoLayerComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	if (EndPlayReason == EEndPlayReason::EndPlayInEditor || EndPlayReason == EEndPlayReason::Quit)
	{
		FStereoLayerAdditionalFlagsManager::Destroy();
	}
}

void UStereoLayerComponent::OnUnregister()
{
	Super::OnUnregister();

	IStereoLayers* StereoLayers;
	if (LayerId && GEngine->StereoRenderingDevice.IsValid() && (StereoLayers = GEngine->StereoRenderingDevice->GetStereoLayers()) != nullptr)
	{
		StereoLayers->DestroyLayer(LayerId);
		LayerId = IStereoLayers::FLayerDesc::INVALID_LAYER_ID;
	}
}

void UStereoLayerComponent::PostLoad()
{
	Super::PostLoad();
	if (Shape == nullptr)
	{
		Shape = NewObject<UStereoLayerShapeQuad>(this, NAME_None, RF_Public);
	}
}

void UStereoLayerComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent( DeltaTime, TickType, ThisTickFunction );

	IStereoLayers* StereoLayers;
	if (!GEngine->StereoRenderingDevice.IsValid() || (StereoLayers = GEngine->StereoRenderingDevice->GetStereoLayers()) == nullptr)
	{
		return;
	}

	FTransform Transform;
	if (StereoLayerType == SLT_WorldLocked)
	{
		Transform = GetComponentTransform();
	}
	else
	{
		Transform = GetRelativeTransform();
	}
	
	// If the transform changed dirty the layer and push the new transform
	if (!bIsDirty && (bLastVisible != GetVisibleFlag() || FMemory::Memcmp(&LastTransform, &Transform, sizeof(Transform)) != 0))
	{
		bIsDirty = true;
	}

	bool bCurrVisible = GetVisibleFlag();
	if ((!Texture || !Texture->GetResource()) && LayerRequiresTexture())
	{
		bCurrVisible = false;
	}

	if (bIsDirty)
	{
		IStereoLayers::FLayerDesc LayerDesc;
		LayerDesc.Priority = Priority;
		LayerDesc.QuadSize = QuadSize;
		LayerDesc.UVRect = UVRect;
		LayerDesc.Transform = Transform;
		
		if (Texture)
		{
			Texture->SetForceMipLevelsToBeResident(30.0f);
			LayerDesc.TextureObj = Texture;
			LayerDesc.Flags |= (Texture->GetMaterialType() == MCT_TextureExternal) ? IStereoLayers::LAYER_FLAG_TEX_EXTERNAL : 0;
		}
		if (LeftTexture)
		{
			LeftTexture->SetForceMipLevelsToBeResident(30.0f);
			LayerDesc.LeftTextureObj = LeftTexture;
		}
				
		LayerDesc.Flags |= (bLiveTexture) ? IStereoLayers::LAYER_FLAG_TEX_CONTINUOUS_UPDATE : 0;
		LayerDesc.Flags |= (bNoAlphaChannel) ? IStereoLayers::LAYER_FLAG_TEX_NO_ALPHA_CHANNEL : 0;
		LayerDesc.Flags |= (bQuadPreserveTextureRatio) ? IStereoLayers::LAYER_FLAG_QUAD_PRESERVE_TEX_RATIO : 0;
		LayerDesc.Flags |= (bSupportsDepth) ? IStereoLayers::LAYER_FLAG_SUPPORT_DEPTH : 0;
		LayerDesc.Flags |= (!bCurrVisible) ? IStereoLayers::LAYER_FLAG_HIDDEN : 0;

		TSharedPtr<FStereoLayerAdditionalFlagsManager> FlagsManager = FStereoLayerAdditionalFlagsManager::Get();
		for (FName& Flag : AdditionalFlags)
		{
			LayerDesc.Flags |= FlagsManager->GetFlagValue(Flag);
		}

		switch (StereoLayerType)
		{
		case SLT_WorldLocked:
			LayerDesc.PositionType = IStereoLayers::WorldLocked;
			break;
		case SLT_TrackerLocked:
			LayerDesc.PositionType = IStereoLayers::TrackerLocked;
			break;
		case SLT_FaceLocked:
			LayerDesc.PositionType = IStereoLayers::FaceLocked;
			break;
		}

		// Set the correct layer shape and apply any shape-specific properties
		if (Shape == nullptr)
		{
			Shape = NewObject<UStereoLayerShapeQuad>(this, NAME_None, RF_Public);
		}
		Shape->ApplyShape(LayerDesc);
		
#if WITH_EDITOR
		LayerDesc.Name = GetOuter()->GetName() + TEXT('_') + GetName();
#endif

		if (LayerId)
		{
			StereoLayers->SetLayerDesc(LayerId, LayerDesc);
		}
		else
		{
			LayerId = StereoLayers->CreateLayer(LayerDesc);
		}
		
		LastTransform = Transform;
		bLastVisible = bCurrVisible;
		bIsDirty = false;
	}

	if (bTextureNeedsUpdate && LayerId)
	{
		StereoLayers->MarkTextureForUpdate(LayerId);
		bTextureNeedsUpdate = false;
	}
}

void UStereoLayerComponent::SetTexture(UTexture* InTexture)
{
	if (Texture == InTexture)
	{
		return;
	}

	Texture = InTexture;
	MarkStereoLayerDirty();
}

void UStereoLayerComponent::SetLeftTexture(UTexture* InTexture)
{
	if (LeftTexture == InTexture)
	{
		return;
	}

	LeftTexture = InTexture;
	MarkStereoLayerDirty();
}

void UStereoLayerComponent::SetQuadSize(FVector2D InQuadSize)
{
	if (QuadSize == InQuadSize)
	{
		return;
	}

	QuadSize = InQuadSize;
	MarkStereoLayerDirty();
}

void UStereoLayerComponent::SetUVRect(FBox2D InUVRect)
{
	if (UVRect == InUVRect)
	{
		return;
	}

	UVRect = InUVRect;
	MarkStereoLayerDirty();
}

void UStereoLayerComponent::SetEquirectProps(FEquirectProps InEquirectProps)
{
	if (Shape->IsA<UStereoLayerShapeEquirect>())
	{
		Cast<UStereoLayerShapeEquirect>(Shape)->SetEquirectProps(InEquirectProps);
	}
}

void UStereoLayerShapeEquirect::SetEquirectProps(FEquirectProps InEquirectProps)
{
	if (InEquirectProps == *this)
	{
		return;
	}
	LeftUVRect = InEquirectProps.LeftUVRect;
	RightUVRect = InEquirectProps.RightUVRect;
	LeftScale = InEquirectProps.LeftScale;
	RightScale = InEquirectProps.RightScale;
	LeftBias = InEquirectProps.LeftBias;
	RightBias = InEquirectProps.RightBias;
	Radius = InEquirectProps.Radius;

	MarkStereoLayerDirty();
}

void UStereoLayerComponent::SetPriority(int32 InPriority)
{
	if (Priority == InPriority)
	{
		return;
	}

	Priority = InPriority;
	MarkStereoLayerDirty();
}

void UStereoLayerShapeCylinder::SetRadius(float InRadius)
{
	if (Radius == InRadius)
	{
		return;
	}

	Radius = InRadius;
	MarkStereoLayerDirty();
}

void UStereoLayerShapeCylinder::SetOverlayArc(float InOverlayArc)
{
	if (OverlayArc == InOverlayArc)
	{
		return;
	}

	OverlayArc = InOverlayArc;
	MarkStereoLayerDirty();
}

void UStereoLayerShapeCylinder::SetHeight(int InHeight)
{
	if (Height == InHeight)
	{
		return;
	}

	Height = InHeight;
	MarkStereoLayerDirty();
}

void UStereoLayerComponent::MarkTextureForUpdate()
{
	bTextureNeedsUpdate = true;
}

void UStereoLayerComponent::MarkStereoLayerDirty()
{
	bIsDirty = true;
}

void UStereoLayerShape::MarkStereoLayerDirty()
{
	check(GetOuter()->IsA<UStereoLayerComponent>());
	Cast<UStereoLayerComponent>(GetOuter())->MarkStereoLayerDirty();
}

void UStereoLayerShapeCylinder::ApplyShape(IStereoLayers::FLayerDesc& LayerDesc)
{
	LayerDesc.SetShape<FCylinderLayer>(Radius, OverlayArc, Height);
}

void UStereoLayerShapeCubemap::ApplyShape(IStereoLayers::FLayerDesc& LayerDesc)
{
	LayerDesc.SetShape<FCubemapLayer>();
}

void UStereoLayerShapeEquirect::ApplyShape(IStereoLayers::FLayerDesc& LayerDesc)
{
	LayerDesc.SetShape<FEquirectLayer>(LeftUVRect, RightUVRect, LeftScale, RightScale, LeftBias, RightBias, Radius);
}

void UStereoLayerShapeQuad::ApplyShape(IStereoLayers::FLayerDesc& LayerDesc)
{
	LayerDesc.SetShape<FQuadLayer>();
}

void UStereoLayerShape::ApplyShape(IStereoLayers::FLayerDesc& LayerDesc)
{
}

#if WITH_EDITOR
void UStereoLayerShape::DrawShapeVisualization(const class FSceneView* View, class FPrimitiveDrawInterface* PDI)
{
}

void UStereoLayerShapeQuad::DrawShapeVisualization(const class FSceneView* View, class FPrimitiveDrawInterface* PDI)
{
	if (!ensure(GEngine) || !ensure(GEngine->EmissiveMeshMaterial))
	{
		return;
	}
	
	FLinearColor YellowColor = FColor(231, 239, 0, 255);
	check(GetOuter()->IsA<UStereoLayerComponent>());

	auto StereoLayerComp = Cast<UStereoLayerComponent>(GetOuter());
	const FVector2D QuadSize = StereoLayerComp->GetQuadSize() / 2.0f;
	const FBox QuadBox(FVector(0.0f, -QuadSize.X, -QuadSize.Y), FVector(0.0f, QuadSize.X, QuadSize.Y));

	DrawWireBox(PDI, StereoLayerComp->GetComponentTransform().ToMatrixWithScale(), QuadBox, YellowColor, 0);

	if (!DebugTextureMaterial)
	{
		DebugTextureMaterial = DuplicateObject<UMaterial>(GEngine->EmissiveMeshMaterial, this);
		DebugTextureMaterial->TwoSided = false;
		DebugTextureMaterial->PostEditChange();
		DebugTextureMaterial->SetTextureParameterValueEditorOnly(NAME_LinearColor, StereoLayerComp->GetTexture());
		
		TArray<FMaterialParameterInfo> OutParameterInfo;
		TArray<FGuid> OutParameterIds;
		DebugTextureMaterial->GetAllTextureParameterInfo(OutParameterInfo, OutParameterIds);
	}
	
	DrawStereoLayerQuad(
		PDI,
		StereoLayerComp->GetComponentTransform().GetLocation(),
		FVector(0, 0, 1),
		FVector(0, 1, 0),
		YellowColor.ToFColor(true),
		StereoLayerComp->GetQuadSize().X,
		StereoLayerComp->GetQuadSize().Y,
		DebugTextureMaterial->GetRenderProxy(),
		0);
}

void UStereoLayerShapeCylinder::DrawShapeVisualization(const class FSceneView* View, class FPrimitiveDrawInterface* PDI)
{
	FLinearColor YellowColor = FColor(231, 239, 0, 255);
	check(GetOuter()->IsA<UStereoLayerComponent>());

	auto StereoLayerComp = Cast<UStereoLayerComponent>(GetOuter());
	float ArcAngle = OverlayArc * 180 / (Radius * UE_PI);

	FVector X = StereoLayerComp->GetComponentTransform().GetUnitAxis(EAxis::Type::X);
	FVector Y = StereoLayerComp->GetComponentTransform().GetUnitAxis(EAxis::Type::Y);
	FVector Base = StereoLayerComp->GetComponentTransform().GetLocation();
	FVector HalfHeight = FVector(0, 0, Height / 2);

	FVector LeftVertex = Base + Radius * (FMath::Cos(ArcAngle / 2 * (UE_PI / 180.0f)) * X + FMath::Sin(ArcAngle / 2 * (UE_PI / 180.0f)) * Y);
	FVector RightVertex = Base + Radius * (FMath::Cos(-ArcAngle / 2 * (UE_PI / 180.0f)) * X + FMath::Sin(-ArcAngle / 2 * (UE_PI / 180.0f)) * Y);

	DrawArc(PDI, Base + HalfHeight, X, Y, -ArcAngle / 2, ArcAngle / 2, Radius, 10, YellowColor, 0);

	DrawArc(PDI, Base - HalfHeight, X, Y, -ArcAngle / 2, ArcAngle / 2, Radius, 10, YellowColor, 0);

	PDI->DrawLine(LeftVertex - HalfHeight, LeftVertex + HalfHeight, YellowColor, 0);

	PDI->DrawLine(RightVertex - HalfHeight, RightVertex + HalfHeight, YellowColor, 0);
}

void UStereoLayerShapeEquirect::DrawShapeVisualization(const class FSceneView* View, class FPrimitiveDrawInterface* PDI)
{
	FLinearColor YellowColor = FColor(231, 239, 0, 255);
	check(GetOuter()->IsA<UStereoLayerComponent>());

	auto StereoLayerComp = Cast<UStereoLayerComponent>(GetOuter());

	DrawWireSphere(PDI, StereoLayerComp->GetComponentTransform().GetTranslation(), YellowColor, (double) Radius, 32, 0);
}
#endif


bool FEquirectProps::operator==(const class UStereoLayerShapeEquirect& Other) const
{
	return (LeftUVRect == Other.LeftUVRect) && (RightUVRect == Other.RightUVRect) && (LeftScale == Other.LeftScale) && (RightScale == Other.RightScale) && (LeftBias == Other.LeftBias) && (RightBias == Other.RightBias) && (Radius == Other.Radius);
}

bool FEquirectProps::operator==(const FEquirectProps& Other) const
{
	return (LeftUVRect == Other.LeftUVRect) && (RightUVRect == Other.RightUVRect) && (LeftScale == Other.LeftScale) && (RightScale == Other.RightScale) && (LeftBias == Other.LeftBias) && (RightBias == Other.RightBias) && (Radius == Other.Radius);
}

TArray<FName> UEditorFlagCollector::GetFlagNames()
{
	TSet<FName> UniqueFlags;
	FStereoLayerAdditionalFlagsManager::CollectFlags(UniqueFlags);
	return UniqueFlags.Array();
}
