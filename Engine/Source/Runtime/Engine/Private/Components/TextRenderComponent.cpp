// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/TextRenderComponent.h"
#include "Internationalization/TextLocalizationManager.h"
#include "SceneView.h"
#include "UObject/ConstructorHelpers.h"
#include "PrimitiveViewRelevance.h"
#include "MaterialDomain.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/Font.h"
#include "Materials/Material.h"
#include "Components/BillboardComponent.h"
#include "Engine/CollisionProfile.h"
#include "Containers/Ticker.h"
#include "SceneInterface.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Texture2D.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMeshBuilder.h"
#include "Engine/TextRenderActor.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "StaticMeshResources.h"
#include "RayTracingInstance.h"
#include "Containers/Ticker.h"
#include "PSOPrecache.h"
#include "DataDrivenShaderPlatformInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextRenderComponent)

static TAutoConsoleVariable<int32> CVarRayTracingTextMeshes(
	TEXT("r.RayTracing.Geometry.Text"),
	1,
	TEXT("Include text meshes in ray tracing effects (default = 1 (text meshes enabled in ray tracing))"));

#define LOCTEXT_NAMESPACE "TextRenderComponent"

ATextRenderActor::ATextRenderActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TextRender = CreateDefaultSubobject<UTextRenderComponent>(TEXT("NewTextRenderComponent"));
	RootComponent = TextRender;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));

	if (!IsRunningCommandlet() && (SpriteComponent != nullptr))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> TextRenderTexture;
			FConstructorStatics()
				: TextRenderTexture(TEXT("/Engine/EditorResources/S_TextRenderActorIcon"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.TextRenderTexture.Get();
		SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->SetupAttachment(TextRender);
		SpriteComponent->bIsScreenSizeScaled = true;
		SpriteComponent->SetUsingAbsoluteScale(true);
		SpriteComponent->bReceivesDecals = false;
	}
#endif
}

// ---------------------------------------------------------------
// Text parsing class which understands line breaks using the '<br>' characters

struct FTextIterator
{
	const TCHAR* SourceString;
	const TCHAR* CurrentPosition;

	FTextIterator(const TCHAR* InSourceString)
		: SourceString(InSourceString)
		, CurrentPosition(InSourceString)
	{
		check(InSourceString);
	}

	bool NextLine()
	{
		check(CurrentPosition);
		return (CurrentPosition[0] != '\0');
	}

	bool NextCharacterInLine(TCHAR& Ch)
	{	
		bool bRet = false;
		check(CurrentPosition);

		if (CurrentPosition[0] == '\0')
		{
			// Leave current position on the null terminator
		}
		else if (CurrentPosition[0] == '<' && CurrentPosition[1] == 'b' && CurrentPosition[2] == 'r' && CurrentPosition[3] == '>')
		{
			CurrentPosition += 4;
		}
		else if (CurrentPosition[0] == '\n')
		{
			++CurrentPosition;
		}
		else
		{
			Ch = *CurrentPosition;
			CurrentPosition++;
			bRet = true;
		}

		return bRet;
	}

	bool Peek(TCHAR& Ch)
	{
		check(CurrentPosition);
		if ( CurrentPosition[0] =='\0' || (CurrentPosition[0] == '<' && CurrentPosition[1] == 'b' && CurrentPosition[2] == 'r' && CurrentPosition[3] == '>') || (CurrentPosition[0] == '\n') )
		{
			return false;
		}
		else
		{
			Ch = CurrentPosition[0];
			return true;
		}
	}
};

// ---------------------------------------------------------------

// @param It must be a valid initialized text iterator
// @param Font 0 is silently ignored
FVector2D ComputeTextSize(FTextIterator It, class UFont* Font,
	float XScale, float YScale, float HorizSpacingAdjust, float VertSpacingAdjust)
{
	FVector2D Ret(0.f, 0.f);

	if(!Font)
	{
		return Ret;
	}

	const float CharIncrement = ( (float)Font->Kerning + HorizSpacingAdjust ) * XScale;

	float LineX = 0.f;

	TCHAR Ch = 0;
	while (It.NextCharacterInLine(Ch))
	{
		Ch = Font->RemapChar(Ch);

		if(!Font->Characters.IsValidIndex(Ch))
		{
			continue;
		}

		FFontCharacter& Char = Font->Characters[Ch];

		if(!Font->Textures.IsValidIndex(Char.TextureIndex))
		{
			continue;
		}

		UTexture2D* Tex = Font->Textures[Char.TextureIndex];

		if(Tex)
		{
			FIntPoint ImportedTextureSize = Tex->GetImportedSize();
			FVector2D InvTextureSize(1.0f / (float)ImportedTextureSize.X, 1.0f / (float)ImportedTextureSize.Y);

			const float X		= LineX;
			const float Y		= Char.VerticalOffset * YScale;
			float SizeX			= Char.USize * XScale;
			const float SizeY	= (Char.VSize + VertSpacingAdjust) * YScale;
			const float U		= Char.StartU * InvTextureSize.X;
			const float V		= Char.StartV * InvTextureSize.Y;
			const float SizeU	= Char.USize * InvTextureSize.X;
			const float SizeV	= Char.VSize * InvTextureSize.Y;

			const float Right = X + SizeX;
			const float Bottom = Y + SizeY;

			Ret.X = FMath::Max(Ret.X, Right);
			Ret.Y = FMath::Max(Ret.Y, Bottom);

			// if we have another non-whitespace character to render, add the font's kerning.
			TCHAR NextCh = 0;
			if( It.Peek(NextCh) && !FChar::IsWhitespace(NextCh) )
			{
				SizeX += CharIncrement;
			}

			LineX += SizeX;
		}
	}

	return Ret;
}

// compute the left top depending on the alignment
static float ComputeHorizontalAlignmentOffset(FVector2D Size, EHorizTextAligment HorizontalAlignment)
{
	float Ret = 0.f;

	switch (HorizontalAlignment)
	{
	case EHTA_Left:
		{
			// X is already 0
			break;
		}

	case EHTA_Center:
		{
			Ret = -Size.X * 0.5f;
			break;
		}

	case EHTA_Right:
		{
			Ret = -Size.X;
			break;
		}

	default:
		{
			// internal error
			check(0);
		}
	}

	return Ret;
}

float ComputeVerticalAlignmentOffset(float SizeY, EVerticalTextAligment VerticalAlignment, float LegacyVerticalOffset)
{
	switch (VerticalAlignment)
	{
	case EVRTA_QuadTop:
		{
			return LegacyVerticalOffset;
		}
	case EVRTA_TextBottom:
		{
			return -SizeY;
		}
	case EVRTA_TextTop:
		{
			return 0.f;
		}
	case EVRTA_TextCenter:
		{
			return -SizeY / 2.0f;
		}
	default:
		{
			check(0);
			return 0.f;
		}
	}
}

/**
 * For the given text info, calculate the vertical offset that needs to be applied to the component
 * in order to vertically align it to the requested alignment.
 **/
float CalculateVerticalAlignmentOffset(
	const TCHAR* Text,
	class UFont* Font,
	float XScale, 
	float YScale, 
	float HorizSpacingAdjust,
	float VertSpacingAdjust,
	EVerticalTextAligment VerticalAlignment)
{
	check(Text);

	if(!Font)
	{
		return 0.f;
	}

	float FirstLineHeight = -1.f; // Only kept around for legacy positioning support
	float StartY = 0.f;

	FTextIterator It(Text);

	while (It.NextLine())
	{
		FVector2D LineSize = ComputeTextSize(It, Font, XScale, YScale, HorizSpacingAdjust, VertSpacingAdjust);

		if (FirstLineHeight < 0.f)
		{
			FirstLineHeight = LineSize.Y;
		}

		// Iterate to end of line
		TCHAR Ch = 0;
		while (It.NextCharacterInLine(Ch)) {}

		// Move Y position down to next line. If the current line is empty, move by max char height in font
		StartY += LineSize.Y > 0.f ? LineSize.Y : Font->GetMaxCharHeight();
	}

	// Calculate a vertical translation to create the correct vertical alignment
	return -ComputeVerticalAlignmentOffset(StartY, VerticalAlignment, FirstLineHeight);
}
	
// ------------------------------------------------------

/** Caches MIDs used by text render components to avoid excessive (re)allocation of MIDs when the SCS runs */
class FTextRenderComponentMIDCache : public FGCObject
{
	FCriticalSection CriticalSection;
public:
	/** Array of MIDs for a particular material and font */
	struct FMIDData
	{
		FMIDData(UMaterialInterface* InMaterial, UFont* InFont)
		{
			check(InMaterial && InFont && InFont->FontCacheType == EFontCacheType::Offline);

			bIsCustomMID = false;

			const int32 NumFontPages = InFont->Textures.Num();

			// Checking IsEngineExitRequested() as a workaround for lighting rebuild command let crash.
			// Happening because IsEngineExitRequested() is true preventing the FTextRenderComponentMIDCache from registering into the GGCObjectReferencer
			if (!IsEngineExitRequested() && NumFontPages > 0)
			{
				TArray<FGuid> FontParameterIds;
				InMaterial->GetAllFontParameterInfo(FontParameters, FontParameterIds);

				if (FontParameters.Num() > 0)
				{
					if (InMaterial->IsA<UMaterialInstanceDynamic>())
					{
						bIsCustomMID = true;

						// If the user provided a custom MID, we can't do anything but use that single MID for page 0
						UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(InMaterial);
						for (const FMaterialParameterInfo& FontParameterInfo : FontParameters)
						{
							MID->SetFontParameterValue(FontParameterInfo, InFont, 0);
						}
						MIDs.Add(ObjectPtrWrap(MID));
					}
					else
					{
						MIDs.Reserve(NumFontPages);
						for (int32 FontPageIndex = 0; FontPageIndex < NumFontPages; ++FontPageIndex)
						{
							UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create(InMaterial, nullptr);
							for (const FMaterialParameterInfo& FontParameterInfo : FontParameters)
							{
								MID->SetFontParameterValue(FontParameterInfo, InFont, FontPageIndex);
							}
							MIDs.Add(ObjectPtrWrap(MID));
						}
					}
				}
			}
		}

		bool IsStale(UMaterialInterface* InMaterial, UFont* InFont) const
		{
			bool bIsStale = false;

			// We can only test for stale MIDs when we created the MIDs ourselves
			if (GIsEditor && !bIsCustomMID)
			{
				// We only test against the number of font pages when we created the MIDs
				bIsStale = MIDs.Num() != InFont->Textures.Num();

				if (!bIsStale)
				{
					TArray<FMaterialParameterInfo> FontParameterInfo;
					TArray<FGuid> FontParameterIds;
					InMaterial->GetAllFontParameterInfo(FontParameterInfo, FontParameterIds);

					bIsStale = FontParameters.Num() != FontParameterInfo.Num();
					for (int32 FontParamIndex = 0; !bIsStale && FontParamIndex < FontParameters.Num(); ++FontParamIndex)
					{
						bIsStale = FontParameters[FontParamIndex] != FontParameterInfo[FontParamIndex];
					}
				}
			}

			return bIsStale;
		}


		TArray<TObjectPtr<UMaterialInstanceDynamic>> MIDs;
		TArray<FMaterialParameterInfo> FontParameters;
		bool bIsCustomMID;
	};

	typedef TSharedRef<const FMIDData, ESPMode::ThreadSafe> FMIDDataRef;
	typedef TSharedPtr<const FMIDData, ESPMode::ThreadSafe> FMIDDataPtr;
	typedef TWeakPtr<const FMIDData, ESPMode::ThreadSafe>   FMIDDataWeakPtr;

	static void Initialize()
	{
		check(!InstancePtr);
		InstancePtr = new FTextRenderComponentMIDCache();
	}

	static void Shutdown()
	{
		delete InstancePtr;
		InstancePtr = nullptr;
	}

	static FTextRenderComponentMIDCache& Get()
	{
		check(InstancePtr);
		return *InstancePtr;
	}

	static bool IsValid()
	{
		return InstancePtr != nullptr;
	}

	FMIDDataRef GetMIDData(UMaterialInterface* InMaterial, UFont* InFont)
	{
		FScopeLock Lock(&CriticalSection);

		check(InMaterial && InFont && InFont->FontCacheType == EFontCacheType::Offline);

		FMIDDataPtr MIDData = CachedMIDs.FindRef(FKey(InMaterial, InFont));
		if (MIDData.IsValid() && MIDData->IsStale(InMaterial, InFont))
		{
			StaleMIDs.Add(MIDData);
			MIDData.Reset();
		}

		if (!MIDData.IsValid())
		{
			MIDData = CachedMIDs.Add(FKey(InMaterial, InFont), MakeShareable(new FMIDData(InMaterial, InFont)));
		}

		return MIDData.ToSharedRef();
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		FScopeLock Lock(&CriticalSection);

		for (auto& MIDDataPair : CachedMIDs)
		{
			FMIDDataPtr& MIDData = MIDDataPair.Value;
			if (!MIDData->bIsCustomMID)
			{
				for (auto& MID : MIDData.Get()->MIDs)
				{
					Collector.AddReferencedObject(ConstCast(MID));
				}
			}
		}

		for (const FMIDDataWeakPtr& StaleMID : StaleMIDs)
		{
			FMIDDataPtr PinnedMID = StaleMID.Pin();
			if (PinnedMID.IsValid() && !PinnedMID->bIsCustomMID)
			{
				for (auto& MID : PinnedMID.Get()->MIDs)
				{
					Collector.AddReferencedObject(ConstCast(MID));
				}
			}
		}
	}

	virtual FString GetReferencerName() const override
	{
		return "FTextRenderComponentMIDCache";
	}

private:
	/** Key identifying an array of MIDs */
	struct FKey
	{
		FKey(const UMaterialInterface* InMaterial, const UFont* InFont)
			: Material(InMaterial)
			, Font(InFont)
		{
		}

		FORCEINLINE bool operator==(const FKey& Other) const
		{
			return Material == Other.Material
				&& Font == Other.Font;
		}

		FORCEINLINE bool operator!=(const FKey& Other) const
		{
			return Material != Other.Material
				|| Font != Other.Font;
		}

		friend inline uint32 GetTypeHash(const FKey& Key)
		{
			return HashCombine(GetTypeHash(Key.Material), GetTypeHash(Key.Font));
		}

		const UMaterialInterface* Material;
		const UFont* Font;
	};

	FTextRenderComponentMIDCache()
	{
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FTextRenderComponentMIDCache::PurgeUnreferencedMIDsTicker), 10.0f);
	}

	~FTextRenderComponentMIDCache()
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}

	bool PurgeUnreferencedMIDsTicker(float)
	{
		PurgeUnreferencedMIDs();
		return true;
	}

	void PurgeUnreferencedMIDs()
	{
		FScopeLock Lock(&CriticalSection);

        QUICK_SCOPE_CYCLE_COUNTER(STAT_FTextRenderComponentMIDCache_PurgeUnreferencedMIDs);

		checkfSlow(IsInGameThread(), TEXT("FTextRenderComponentMIDCache::PurgeUnreferencedMIDs is only expected to be called from the game thread!"));

		TArray<FKey> MIDsToPurgeNow;
		TSet<FKey> MIDsToPurgeLater;

		for (const auto& MIDDataPair : CachedMIDs)
		{
			const FMIDDataPtr& MIDData = MIDDataPair.Value;
			if (MIDData.IsUnique())
			{
				if (MIDsPendingPurge.Contains(MIDDataPair.Key))
				{
					MIDsToPurgeNow.Add(MIDDataPair.Key);
				}
				else
				{
					MIDsToPurgeLater.Add(MIDDataPair.Key);
				}
			}
		}

		for (const FKey& MIDKey : MIDsToPurgeNow)
		{
			CachedMIDs.Remove(MIDKey);
		}

		StaleMIDs.RemoveAll([](const FMIDDataWeakPtr& InStaleMID)
		{
			return !InStaleMID.IsValid();
		});

		MIDsPendingPurge = MoveTemp(MIDsToPurgeLater);
	}

	FTSTicker::FDelegateHandle TickerHandle;

	TMap<FKey, FMIDDataPtr> CachedMIDs;

	TArray<FMIDDataWeakPtr> StaleMIDs;

	TSet<FKey> MIDsPendingPurge;

	static FTextRenderComponentMIDCache* InstancePtr;
};

FTextRenderComponentMIDCache* FTextRenderComponentMIDCache::InstancePtr = nullptr;

// ------------------------------------------------------

/** Represents a URenderTextComponent to the scene manager. */
class FTextRenderSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	// constructor / destructor
	FTextRenderSceneProxy(UTextRenderComponent* Component);
	virtual ~FTextRenderSceneProxy();

	// Begin FPrimitiveSceneProxy interface
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual bool CanBeOccluded() const override;
	virtual bool IsUsingDistanceCullFade() const override;
	virtual uint32 GetMemoryFootprint() const override;
	uint32 GetAllocatedSize() const;
#if RHI_RAYTRACING
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override final;
	virtual bool HasRayTracingRepresentation() const override { return true; }
	virtual bool IsRayTracingRelevant() const override { return true; }
	virtual bool IsRayTracingStaticRelevant() const override
	{
		return false;
	}
#endif
	// End FPrimitiveSceneProxy interface

private:
	// Begin FPrimitiveSceneProxy interface
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	// End FPrimitiveSceneProxy interface

	void ReleaseRenderThreadResources();
	bool BuildStringMesh( TArray<FDynamicMeshVertex>& OutVertices, TArray<uint32>& OutIndices );

private:
	struct FTextBatch
	{
		int32 IndexBufferOffset;
		int32 IndexBufferCount;

		int32 VertexBufferOffset;
		int32 VertexBufferCount;

		const UMaterialInterface* Material;
	};

	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
	FLocalVertexFactory VertexFactory;
	TArray<FTextBatch> TextBatches;
	const FColor TextRenderColor;
	FMaterialRelevance MaterialRelevance;
	UMaterialInterface* TextMaterial;
	UFont* Font;
	FTextRenderComponentMIDCache::FMIDDataPtr FontMIDs;
	FText Text;
	float XScale;
	float YScale;
	float HorizSpacingAdjust;
	float VertSpacingAdjust;
	TEnumAsByte<EHorizTextAligment> HorizontalAlignment;
	TEnumAsByte<EVerticalTextAligment> VerticalAlignment;
	bool bAlwaysRenderAsText;

#if RHI_RAYTRACING
	/** Geometry for ray tracing. */
	FRayTracingGeometry RayTracingGeometry;
	TArray<FMeshBatch> CachedRayTracingMaterials;

	bool bSupportRayTracing;
	bool bRayTracingWithWPO;
	bool bNeedsToUpdateRayTracingCache;

	void UpdateRayTracingGeometry_RenderingThread(FRHICommandListBase& RHICmdList);
#endif
};

FTextRenderSceneProxy::FTextRenderSceneProxy( UTextRenderComponent* Component) :
	FPrimitiveSceneProxy(Component),
	VertexFactory(GetScene().GetFeatureLevel(), "FTextRenderSceneProxy"),
	TextRenderColor(Component->TextRenderColor),
	Font(Component->Font),
	Text(Component->Text),
	XScale(Component->WorldSize * Component->XScale * Component->InvDefaultSize),
	YScale(Component->WorldSize * Component->YScale * Component->InvDefaultSize),
	HorizSpacingAdjust(Component->HorizSpacingAdjust),
	VertSpacingAdjust(Component->VertSpacingAdjust),
	HorizontalAlignment(Component->HorizontalAlignment),
	VerticalAlignment(Component->VerticalAlignment),
	bAlwaysRenderAsText(Component->bAlwaysRenderAsText)
{
	SetWireframeColor(FLinearColor(1.f, 0.f, 0.f));
	UMaterialInterface* EffectiveMaterial = nullptr;

	if(Component->TextMaterial)
	{
		const UMaterial* BaseMaterial = Component->TextMaterial->GetMaterial_Concurrent();

		if(BaseMaterial->MaterialDomain == MD_Surface || BaseMaterial->MaterialDomain == MD_DeferredDecal)
		{
			EffectiveMaterial = Component->TextMaterial;
		}
	}

	if(!EffectiveMaterial)
	{
		EffectiveMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}

	TextMaterial = EffectiveMaterial;
	MaterialRelevance |= TextMaterial->GetMaterial_Concurrent()->GetRelevance(GetScene().GetShaderPlatform());

	if (Font && Font->FontCacheType == EFontCacheType::Offline)
	{
		if (FTextRenderComponentMIDCache::IsValid())
		{
			FontMIDs = FTextRenderComponentMIDCache::Get().GetMIDData(TextMaterial, Font);
		}
	}

	// The MID from the cache isn't known by the UTextRenderComponent
	bVerifyUsedMaterials = false;

#if RHI_RAYTRACING
	bSupportRayTracing = IsRayTracingEnabled();
	bNeedsToUpdateRayTracingCache = true;
	bRayTracingWithWPO = MaterialRelevance.bUsesWorldPositionOffset;
#endif
}

FTextRenderSceneProxy::~FTextRenderSceneProxy()
{
	ReleaseRenderThreadResources();
}

void FTextRenderSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	if (Font && Font->FontCacheType == EFontCacheType::Runtime)
	{
		// Runtime fonts can't currently be used here as they use the font cache from Slate application
		// which can only be used on the game thread
		return;
	}

	TArray<FDynamicMeshVertex> OutVertices;
	if (BuildStringMesh(OutVertices, IndexBuffer.Indices))
	{
#if RHI_ENABLE_RESOURCE_INFO
		FName Name = FName(TEXT("FTextRenderSceneProxy ") + GetOwnerName().ToString());
		VertexBuffers.SetOwnerName(Name);
		IndexBuffer.SetOwnerName(Name);
#endif
#if RHI_RAYTRACING
		EnumAddFlags(IndexBuffer.UsageFlags, EBufferUsageFlags::ShaderResource);
#endif
		VertexBuffers.InitFromDynamicVertex(RHICmdList, &VertexFactory, OutVertices);
		// Enqueue initialization of render resources
		IndexBuffer.InitResource(RHICmdList);

#if RHI_RAYTRACING
		if (bSupportRayTracing)
		{
			UpdateRayTracingGeometry_RenderingThread(RHICmdList);
			bNeedsToUpdateRayTracingCache = true;
		}
#endif
	}
}

void FTextRenderSceneProxy::ReleaseRenderThreadResources()
{
	VertexBuffers.PositionVertexBuffer.ReleaseResource();
	VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
	VertexBuffers.ColorVertexBuffer.ReleaseResource();
	IndexBuffer.ReleaseResource();
	VertexFactory.ReleaseResource();

#if RHI_RAYTRACING
	RayTracingGeometry.ReleaseResource();
#endif
}

void FTextRenderSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const 
{
	QUICK_SCOPE_CYCLE_COUNTER( STAT_TextRenderSceneProxy_GetDynamicMeshElements );

	// Vertex factory will not been initialized when the text string is empty or font is invalid.
	if(VertexFactory.IsInitialized())
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];

				for (const FTextBatch& TextBatch : TextBatches)
				{
					// Draw the mesh.
					FMeshBatch& Mesh = Collector.AllocateMesh();
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement.IndexBuffer = &IndexBuffer;
					Mesh.VertexFactory = &VertexFactory;
					BatchElement.FirstIndex = TextBatch.IndexBufferOffset;
					BatchElement.NumPrimitives = TextBatch.IndexBufferCount / 3;
					BatchElement.MinVertexIndex = TextBatch.VertexBufferOffset;
					BatchElement.MaxVertexIndex = TextBatch.VertexBufferOffset + TextBatch.VertexBufferCount - 1;
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
					Mesh.bDisableBackfaceCulling = false;
					Mesh.Type = PT_TriangleList;
					Mesh.DepthPriorityGroup = SDPG_World;
					Mesh.MaterialRenderProxy = TextBatch.Material->GetRenderProxy();
					Mesh.bCanApplyViewModeOverrides = !bAlwaysRenderAsText;
					Mesh.LODIndex = 0;
					Mesh.bUseWireframeSelectionColoring = IsSelected() ? 1 : 0;

					Collector.AddMesh(ViewIndex, Mesh);
				}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				RenderBounds(Collector.GetPDI(ViewIndex), View->Family->EngineShowFlags, GetBounds(), IsSelected());
#endif
			}
		}
	}
}

void FTextRenderSceneProxy::DrawStaticElements(FStaticPrimitiveDrawInterface* PDI)
{
	// Vertex factory will not been initialized when the font is invalid or the text string is empty.
	if(VertexFactory.IsInitialized())
	{
		PDI->ReserveMemoryForMeshes(TextBatches.Num());

		for (const FTextBatch& TextBatch : TextBatches)
		{
			// Draw the mesh.
			FMeshBatch Mesh;
			FMeshBatchElement& BatchElement = Mesh.Elements[0];
			BatchElement.IndexBuffer = &IndexBuffer;
			Mesh.VertexFactory = &VertexFactory;
			Mesh.MaterialRenderProxy = TextBatch.Material->GetRenderProxy();
			BatchElement.FirstIndex = TextBatch.IndexBufferOffset;
			BatchElement.NumPrimitives = TextBatch.IndexBufferCount / 3;
			BatchElement.MinVertexIndex = TextBatch.VertexBufferOffset;
			BatchElement.MaxVertexIndex = TextBatch.VertexBufferOffset + TextBatch.VertexBufferCount - 1;
			Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
			Mesh.bDisableBackfaceCulling = false;
			Mesh.Type = PT_TriangleList;
			Mesh.DepthPriorityGroup = SDPG_World;
			Mesh.LODIndex = 0;
			PDI->DrawMesh(Mesh, 1.0f);
		}
	}
}

bool FTextRenderSceneProxy::CanBeOccluded() const
{
	return !MaterialRelevance.bDisableDepthTest;
}

bool FTextRenderSceneProxy::IsUsingDistanceCullFade() const
{
	return MaterialRelevance.bUsesDistanceCullFade;
}

FPrimitiveViewRelevance FTextRenderSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.TextRender;
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;

	if( IsRichView(*View->Family) 
		|| View->Family->EngineShowFlags.Bounds 
		|| View->Family->EngineShowFlags.Collision 
		|| IsSelected() 
		|| IsHovered()
		)
	{
		Result.bDynamicRelevance = true;
	}
	else
	{
		Result.bStaticRelevance = true;
	}

	MaterialRelevance.SetPrimitiveViewRelevance(Result);
	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
	return Result;
}

uint32 FTextRenderSceneProxy::GetMemoryFootprint() const
{
	return( sizeof( *this ) + GetAllocatedSize() );
}

uint32 FTextRenderSceneProxy::GetAllocatedSize() const
{
	return( FPrimitiveSceneProxy::GetAllocatedSize() ); 
}

#if RHI_RAYTRACING
void FTextRenderSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	if (CVarRayTracingTextMeshes.GetValueOnRenderThread() == 0 || !bSupportRayTracing)
	{
		return;
	}
	// Vertex factory will not been initialized when the text string is empty or font is invalid.
	if (!VertexFactory.IsInitialized())
	{
		return;
	}

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const FSceneView* FirstActiveView = Views[FirstActiveViewIndex]; // TODO: should use view-specific instances?

	FRayTracingInstance RayTracingInstance;

	if (bNeedsToUpdateRayTracingCache)
	{
		CachedRayTracingMaterials.Reset();
		int NumBatches = TextBatches.Num();
		for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
		{
			const FTextBatch& TextBatch = TextBatches[BatchIndex];
			FMeshBatch& MeshBatch = CachedRayTracingMaterials.AddDefaulted_GetRef();
			FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
			BatchElement.IndexBuffer = &IndexBuffer;
			MeshBatch.VertexFactory = &VertexFactory;
			BatchElement.FirstIndex = TextBatch.IndexBufferOffset;
			BatchElement.NumPrimitives = TextBatch.IndexBufferCount / 3;
			BatchElement.MinVertexIndex = TextBatch.VertexBufferOffset;
			BatchElement.MaxVertexIndex = TextBatch.VertexBufferOffset + TextBatch.VertexBufferCount - 1;
			MeshBatch.bDisableBackfaceCulling = false;
			MeshBatch.Type = PT_TriangleList;
			MeshBatch.DepthPriorityGroup = SDPG_World;
			MeshBatch.MaterialRenderProxy = TextBatch.Material->GetRenderProxy();
			MeshBatch.bCanApplyViewModeOverrides = !bAlwaysRenderAsText;
			MeshBatch.CastRayTracedShadow = IsShadowCast(FirstActiveView);
			MeshBatch.LODIndex = 0;
			MeshBatch.bUseWireframeSelectionColoring = IsSelected() ? 1 : 0;
			MeshBatch.SegmentIndex = BatchIndex;
			MeshBatch.MeshIdInPrimitive = 0;
		}
		bNeedsToUpdateRayTracingCache = false;
	}
	else
	{
		RayTracingInstance.bInstanceMaskAndFlagsDirty = false;
	}

	RayTracingInstance.MaterialsView = MakeArrayView(CachedRayTracingMaterials);
	RayTracingInstance.Geometry = &RayTracingGeometry;
	const FMatrix& ThisLocalToWorld = GetLocalToWorld();
	RayTracingInstance.InstanceTransformsView = MakeArrayView(&ThisLocalToWorld, 1);

	if (bRayTracingWithWPO && VertexFactory.GetType()->SupportsRayTracingDynamicGeometry())
	{
		FRayTracingDynamicGeometryUpdateParams UpdateParams;
		UpdateParams.MeshBatchesView = CachedRayTracingMaterials;
		UpdateParams.NumVertices = (uint32)VertexBuffers.PositionVertexBuffer.GetNumVertices();
		UpdateParams.VertexBufferSize = uint32((SIZE_T)VertexBuffers.PositionVertexBuffer.GetNumVertices() * sizeof(FVector3f));
		UpdateParams.NumTriangles = RayTracingGeometry.GetInitializer().TotalPrimitiveCount;
		UpdateParams.Geometry = &RayTracingGeometry;

		Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));
	}
	else if (RayTracingGeometry.IsEvicted())
	{
		Collector.AddReferencedGeometry(RayTracingGeometry.GetGeometryHandle());
		return;
	}

	check(CachedRayTracingMaterials.Num() == RayTracingInstance.GetMaterials().Num());
	checkf(RayTracingInstance.Geometry->GetInitializer().Segments.Num() == CachedRayTracingMaterials.Num(), TEXT("Segments/Materials mismatch. Number of segments: %d. Number of Materials: %d."),
		RayTracingInstance.Geometry->GetInitializer().Segments.Num(),
		CachedRayTracingMaterials.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if ((VisibilityMap & (1 << ViewIndex)) == 0)
		{
			continue;
		}

		Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
	}

}
#endif // RHI_RAYTRACING

/**
* For the given text, constructs a mesh to be used by the vertex factory for rendering.
*/
bool FTextRenderSceneProxy::BuildStringMesh( TArray<FDynamicMeshVertex>& OutVertices, TArray<uint32>& OutIndices )
{
	TextBatches.Reset();

	if(!Font || Text.IsEmpty())
	{
		return false;
	}

	float FirstLineHeight = -1.f; // Only kept around for legacy positioning support
	float StartY = 0.f;

	const float CharIncrement = ( (float)Font->Kerning + HorizSpacingAdjust ) * XScale;

	float LineX = 0.f;

	int32 PageIndex = INDEX_NONE;

	int32 FirstVertexIndexInTextBatch = 0;
	int32 FirstIndiceIndexInTextBatch = 0;
	auto FinishTextBatch = [&]()
	{
		if (OutVertices.Num() > FirstVertexIndexInTextBatch && OutIndices.Num() > FirstIndiceIndexInTextBatch)
		{
			FTextBatch& TextBatch = TextBatches[TextBatches.AddUninitialized()];

			TextBatch.IndexBufferOffset = FirstIndiceIndexInTextBatch;
			TextBatch.IndexBufferCount = OutIndices.Num() - FirstIndiceIndexInTextBatch;

			TextBatch.VertexBufferOffset = FirstVertexIndexInTextBatch;
			TextBatch.VertexBufferCount = OutVertices.Num() - FirstVertexIndexInTextBatch;

			TextBatch.Material = nullptr;
			if (FontMIDs.IsValid() && FontMIDs->MIDs.IsValidIndex(PageIndex))
			{
				TextBatch.Material = FontMIDs->MIDs[PageIndex];
			}
			if (!TextBatch.Material)
			{
				TextBatch.Material = TextMaterial;
			}
		}

		FirstVertexIndexInTextBatch = OutVertices.Num();
		FirstIndiceIndexInTextBatch = OutIndices.Num();
	};

	// TArray uses int32 for size so it cannot store more than MAX_int32 elements.
	const uint64 MaxVerts = MAX_int32 - OutVertices.Num();
	const uint64 MaxIndices = MAX_int32 - OutIndices.Num();
	const uint64 MaxQuads = FMath::Min(MaxVerts / 4, MaxIndices / 6);

	// We have one quad per glyph, so presize the index and vertex buffer.
	const FString& Str = Text.ToString();
	const uint64 NumQuads = FMath::Min((uint64)Str.Len(), MaxQuads);
	OutVertices.Reserve(OutVertices.Num() + NumQuads * 4);
	OutIndices.Reserve(OutIndices.Num() + NumQuads * 6);

	uint64 QuadIdx = 0;
	FTextIterator It(*Str);
	while (It.NextLine() && QuadIdx < NumQuads)
	{
		FVector2D LineSize = ComputeTextSize(It, Font, XScale, YScale, HorizSpacingAdjust, VertSpacingAdjust);
		float StartX = ComputeHorizontalAlignmentOffset(LineSize, HorizontalAlignment);

		if (FirstLineHeight < 0.f)
		{
			FirstLineHeight = LineSize.Y;
		}

		LineX = 0.f;

		TCHAR Ch = 0;
		while (It.NextCharacterInLine(Ch) && QuadIdx < NumQuads)
		{
			Ch = Font->RemapChar(Ch);

			if(!Font->Characters.IsValidIndex(Ch))
			{
				continue;
			}

			FFontCharacter& Char = Font->Characters[Ch];

			if(!Font->Textures.IsValidIndex(Char.TextureIndex))
			{
				continue;
			}

			// Need to start a new batch?
			if (PageIndex != Char.TextureIndex)
			{
				if (PageIndex != INDEX_NONE)
				{
					FinishTextBatch();
				}

				PageIndex = Char.TextureIndex;
			}

			UTexture2D* Tex = Font->Textures[Char.TextureIndex];

			if(Tex)
			{
				FIntPoint ImportedTextureSize = Tex->GetImportedSize();
				FVector2f InvTextureSize(1.0f / (float)ImportedTextureSize.X, 1.0f / (float)ImportedTextureSize.Y);

				const float X		= LineX + StartX;
				const float Y		= StartY + Char.VerticalOffset * YScale;
				const float SizeX	= Char.USize * XScale;
				const float SizeY	= Char.VSize * YScale;
				const float U		= Char.StartU * InvTextureSize.X;
				const float V		= Char.StartV * InvTextureSize.Y;
				const float SizeU	= Char.USize * InvTextureSize.X;
				const float SizeV	= Char.VSize * InvTextureSize.Y;			

				const float Left = X;
				const float Top = Y;
				const float Right = X + SizeX;
				const float Bottom = Y + SizeY;

				// axis choice and sign to get good alignment when placed on surface
				const FVector4f V0(0.f, -Left, -Top, 0.f);
				const FVector4f V1(0.f, -Right, -Top, 0.f);
				const FVector4f V2(0.f, -Left, -Bottom, 0.f);
				const FVector4f V3(0.f, -Right, -Bottom, 0.f);

				const FVector3f TangentX(0.f, -1.f, 0.f);
				const FVector3f TangentY(0.f, 0.f, -1.f);
				const FVector3f TangentZ(1.f, 0.f, 0.f);

				const int32 V00 = OutVertices.Add(FDynamicMeshVertex(V0, TangentX, TangentZ, FVector2f(U, V), TextRenderColor));
				const int32 V10 = OutVertices.Add(FDynamicMeshVertex(V1, TangentX, TangentZ, FVector2f(U + SizeU, V), TextRenderColor));
				const int32 V01 = OutVertices.Add(FDynamicMeshVertex(V2, TangentX, TangentZ, FVector2f(U, V + SizeV), TextRenderColor));
				const int32 V11 = OutVertices.Add(FDynamicMeshVertex(V3, TangentX, TangentZ, FVector2f(U + SizeU, V + SizeV), TextRenderColor));

				OutIndices.Add(V00);
				OutIndices.Add(V11);
				OutIndices.Add(V10);

				OutIndices.Add(V00);
				OutIndices.Add(V01);
				OutIndices.Add(V11);

				LineX += SizeX;

				// if we have another non-whitespace character to render, add the font's kerning.
				TCHAR NextChar = 0;
				if( It.Peek(NextChar) && !FChar::IsWhitespace(NextChar) )
				{
					LineX += CharIncrement;
				}

				++QuadIdx;
			}
		}

		// Move Y position down to next line. If the current line is empty, move by max char height in font
		StartY += LineSize.Y > 0.f ? LineSize.Y : Font->GetMaxCharHeight();
	}

	FinishTextBatch();

	// Avoid initializing RHI resources when no vertices are generated.
	return (OutVertices.Num() > 0);
}

#if RHI_RAYTRACING
void FTextRenderSceneProxy::UpdateRayTracingGeometry_RenderingThread(FRHICommandListBase& RHICmdList)
{
	static const FName DebugName("FTextRenderSceneProxy");
	static int32 DebugNumber = 0;

	// Creating ray tracing geometries for text
	FRayTracingGeometryInitializer Initializer;
	Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
	Initializer.IndexBuffer = IndexBuffer.IndexBufferRHI;
	Initializer.TotalPrimitiveCount = IndexBuffer.Indices.Num() / 3;
	Initializer.GeometryType = RTGT_Triangles;
	Initializer.bFastBuild = true;
	Initializer.bAllowUpdate = false;

	for (int BatchIndex = 0; BatchIndex < TextBatches.Num(); ++BatchIndex)
	{
		const FTextBatch& TextBatch = TextBatches[BatchIndex];
		FRayTracingGeometrySegment Segment;
		Segment.VertexBuffer = VertexBuffers.PositionVertexBuffer.VertexBufferRHI;
		Segment.VertexBufferElementType = VET_Float3;
		Segment.VertexBufferStride = VertexBuffers.PositionVertexBuffer.GetStride();
		Segment.VertexBufferOffset = 0;
		Segment.MaxVertices = VertexBuffers.PositionVertexBuffer.GetNumVertices();
		Segment.FirstPrimitive = TextBatch.IndexBufferOffset / 3;
		Segment.NumPrimitives =  TextBatch.IndexBufferCount / 3;
		Segment.bEnabled = true;
		Segment.bForceOpaque = false;

		Initializer.Segments.Add(Segment);
	}

	RayTracingGeometry.SetInitializer(Initializer);
	RayTracingGeometry.InitResource(RHICmdList);
}
#endif

// ------------------------------------------------------

/** Watches for culture changes and updates all live UTextRenderComponent components */
class FTextRenderComponentCultureChangedFixUp
{
public:
	FTextRenderComponentCultureChangedFixUp()
		: ImplPtr(MakeShareable(new FImpl()))
	{
		ImplPtr->Register();
	}

private:
	struct FImpl : public TSharedFromThis<FImpl>
	{
		void Register()
		{
			FTextLocalizationManager::Get().OnTextRevisionChangedEvent.AddSP(this, &FImpl::HandleLocalizedTextChanged);
		}

		void HandleLocalizedTextChanged()
		{
			for (UTextRenderComponent* TextRenderComponent : TObjectRange<UTextRenderComponent>())
			{
				TextRenderComponent->MarkRenderStateDirty();
			}
		}
	};

	TSharedPtr<FImpl> ImplPtr;
};

// ------------------------------------------------------

UTextRenderComponent::UTextRenderComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if( !IsRunningDedicatedServer() )
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UFont> Font;
			ConstructorHelpers::FObjectFinderOptional<UMaterial> TextMaterial;
			FConstructorStatics()
				: Font(TEXT("/Engine/EngineFonts/RobotoDistanceField"))
				, TextMaterial(TEXT("/Engine/EngineMaterials/DefaultTextMaterialOpaque"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		{
			// Static used to watch for culture changes and update all live UTextRenderComponent components
			// In this constructor so that it has a known initialization order, and is only created when we need it
			static FTextRenderComponentCultureChangedFixUp TextRenderComponentCultureChangedFixUp;
		}

		PrimaryComponentTick.bCanEverTick = false;
		bTickInEditor = false;

		Text = LOCTEXT("DefaultText", "Text");

		Font = ConstructorStatics.Font.Get();
		TextMaterial = ConstructorStatics.TextMaterial.Get();

		SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		TextRenderColor = FColor::White;
		XScale = 1.f;
		YScale = 1.f;
		HorizSpacingAdjust = 0.f;
		VertSpacingAdjust = 0.f;
		HorizontalAlignment = EHTA_Left;
		VerticalAlignment = EVRTA_TextBottom;

		SetGenerateOverlapEvents(false);

		if(Font)
		{
			Font->ConditionalPostLoad();
			WorldSize = Font->GetMaxCharHeight();
			InvDefaultSize = 1.0f / WorldSize;
		}
		else
		{
			WorldSize = 30.0f;
			InvDefaultSize = 1.0f / 30.0f;
		}
	}
}

FPrimitiveSceneProxy* UTextRenderComponent::CreateSceneProxy()
{
	return new FTextRenderSceneProxy(this);
}

void UTextRenderComponent::GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials ) const
{
	OutMaterials.Add(TextMaterial);
}

int32 UTextRenderComponent::GetNumMaterials() const
{
	return 1;
}

#if WITH_EDITOR
bool UTextRenderComponent::GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty)
{
	if (ElementIndex == 0)
	{
		OutOwner = this;
		OutPropertyPath = GET_MEMBER_NAME_STRING_CHECKED(UTextRenderComponent, TextMaterial);
		OutProperty = UTextRenderComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTextRenderComponent, TextMaterial));
		return true;
	}

	return false;
}
#endif // WITH_EDITOR

void UTextRenderComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* InMaterial)
{
	if (ElementIndex == 0)
	{
		SetTextMaterial(InMaterial);
	}
}

UMaterialInterface* UTextRenderComponent::GetMaterial(int32 ElementIndex) const
{
	return (ElementIndex == 0) ? TextMaterial : nullptr;
}

bool UTextRenderComponent::ShouldRecreateProxyOnUpdateTransform() const
{
	// We used to rebuild the text every time we moved it, but
	// now we rely on transforms, so it is no longer necessary.
	return false;
}

FBoxSphereBounds UTextRenderComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if(!Text.IsEmpty() && Font)
	{
		FVector2D Size(FLT_MIN, 0.f);
		FVector2D LeftTop(FLT_MAX, FLT_MAX);
		float FirstLineHeight = -1.f;

		FTextIterator It(*Text.ToString());

		const float AdjustedXScale = WorldSize * XScale * InvDefaultSize;
		const float AdjustedYScale = WorldSize * YScale * InvDefaultSize;

		while (It.NextLine())
		{
			const FVector2D LineSize = ComputeTextSize(It, Font, AdjustedXScale, AdjustedYScale, HorizSpacingAdjust, VertSpacingAdjust);
			const float LineLeft = ComputeHorizontalAlignmentOffset(LineSize, HorizontalAlignment);

			Size.X = FMath::Max(LineSize.X, Size.X);
			Size.Y += LineSize.Y > 0.f ? LineSize.Y : Font->GetMaxCharHeight();
			LeftTop.X = FMath::Min(LeftTop.X, LineLeft);

			if (FirstLineHeight < 0.f)
			{
				FirstLineHeight = LineSize.Y;
			}

			TCHAR Ch = 0;
			while (It.NextCharacterInLine(Ch)) {}
		}

		const FBox LocalBox(FVector(0.f, -Size.X - LeftTop.X, -Size.Y), FVector(0.f, -LeftTop.X, 0.0f));
		FBoxSphereBounds Ret(LocalBox.TransformBy(LocalToWorld));

		Ret.BoxExtent *= BoundsScale;
		Ret.SphereRadius *= BoundsScale;

		return Ret;
	}
	else
	{
		return FBoxSphereBounds(ForceInit).TransformBy(LocalToWorld);
	}
}

void UTextRenderComponent::UpdateBounds()
{
	Bounds = CalcBounds(FTransform(GetRenderMatrix()));
}

bool UTextRenderComponent::RequiresGameThreadEndOfFrameUpdates() const
{
	return true;
}

FMatrix UTextRenderComponent::GetRenderMatrix() const
{
	// Adjust LocalToWorld transform to account for vertical text alignment when rendering.
	if(!Text.IsEmpty() && Font)
	{
		float SizeY = 0.f;
		float FirstLineHeight = -1.f;
		const float AdjustedXScale = WorldSize * XScale * InvDefaultSize;
		const float AdjustedYScale = WorldSize * YScale * InvDefaultSize;

		FTextIterator It(*Text.ToString());
		while (It.NextLine())
		{
			const FVector2D LineSize = ComputeTextSize(It, Font, AdjustedXScale, AdjustedYScale, HorizSpacingAdjust, VertSpacingAdjust);
			SizeY += LineSize.Y > 0.f ? LineSize.Y : Font->GetMaxCharHeight();

			if (FirstLineHeight < 0.f)
			{
				FirstLineHeight = LineSize.Y;
			}

			TCHAR Ch = 0;
			while (It.NextCharacterInLine(Ch)) {}
		}

		// Calculate a vertical translation to create the correct vertical alignment
		FMatrix VerticalTransform = FMatrix::Identity;
		const float VerticalAlignmentOffset = -ComputeVerticalAlignmentOffset(SizeY, VerticalAlignment, FirstLineHeight);
		VerticalTransform = VerticalTransform.ConcatTranslation(FVector(0.f, 0.f, VerticalAlignmentOffset));

		return VerticalTransform *  GetComponentTransform().ToMatrixWithScale();

	}
	return UPrimitiveComponent::GetRenderMatrix();
}

void UTextRenderComponent::SetText(const FText& Value)
{
	K2_SetText(Value);
}

void UTextRenderComponent::K2_SetText(const FText& Value)
{
	Text = Value;
	MarkRenderStateDirty();
}

void UTextRenderComponent::SetTextMaterial(class UMaterialInterface* Value)
{
	TextMaterial = Value;
	MarkRenderStateDirty();	
}

void UTextRenderComponent::SetFont(UFont* Value)
{
	Font = Value;
	if( Font )
	{
		InvDefaultSize = 1.0f / Font->GetMaxCharHeight();
	}
	else
	{
		InvDefaultSize = 1.0f;
	}

	MarkRenderStateDirty();	
}

void UTextRenderComponent::SetHorizontalAlignment(EHorizTextAligment Value)
{
	HorizontalAlignment = Value;
	MarkRenderStateDirty();	
}

void UTextRenderComponent::SetVerticalAlignment(EVerticalTextAligment Value)
{
	VerticalAlignment = Value;
	MarkRenderStateDirty();
}

void UTextRenderComponent::SetTextRenderColor(FColor Value)
{
	TextRenderColor = Value;
	MarkRenderStateDirty();	
}

void UTextRenderComponent::SetXScale(float Value)
{
	XScale = Value;
	MarkRenderStateDirty();	
}

void UTextRenderComponent::SetYScale(float Value)
{
	YScale = Value;
	MarkRenderStateDirty();	
}

void UTextRenderComponent::SetHorizSpacingAdjust(float Value)
{
	HorizSpacingAdjust = Value;
	MarkRenderStateDirty();	
}

void UTextRenderComponent::SetVertSpacingAdjust(float Value)
{
	VertSpacingAdjust = Value;
	MarkRenderStateDirty();	
}

void UTextRenderComponent::SetWorldSize(float Value)
{
	WorldSize = Value;
	MarkRenderStateDirty();	
}

FVector UTextRenderComponent::GetTextLocalSize() const
{
	const FBoxSphereBounds TextBounds = CalcBounds(FTransform::Identity);
	return TextBounds.GetBox().GetSize();
}

FVector UTextRenderComponent::GetTextWorldSize() const
{
	const FBoxSphereBounds TextBounds = CalcBounds(GetComponentTransform());
	return TextBounds.GetBox().GetSize();
}

void UTextRenderComponent::PostLoad()
{
	// Try and fix up assets created before the vertical alignment fix was implemented. Because we didn't flag that
	// fix with its own version, use the version number closest to that CL
	if (GetLinkerUEVersion() < VER_UE4_PACKAGE_REQUIRES_LOCALIZATION_GATHER_FLAGGING)
	{
		const float Offset = CalculateVerticalAlignmentOffset(*Text.ToString(), Font, XScale, YScale, HorizSpacingAdjust, VertSpacingAdjust, VerticalAlignment);
		const FTransform RelativeTransform = GetRelativeTransform();
		FTransform CorrectionLeft = FTransform::Identity;
		FTransform CorrectionRight = FTransform::Identity;
		CorrectionLeft.SetTranslation(FVector(0.0f, 0.0f, -Offset));
		CorrectionRight.SetTranslation(FVector(0.0f, 0.0f, Offset));
		SetRelativeTransform(CorrectionLeft * RelativeTransform * CorrectionRight);
	}

	if (GetLinkerUEVersion() < VER_UE4_ADD_TEXT_COMPONENT_VERTICAL_ALIGNMENT)
	{
		VerticalAlignment = EVRTA_QuadTop;
	}

	if( GetLinkerUEVersion() < VER_UE4_TEXT_RENDER_COMPONENTS_WORLD_SPACE_SIZING )
	{
		if( Font )
		{
			WorldSize = Font->GetMaxCharHeight();
			InvDefaultSize = 1.0f / WorldSize;
		}
		else
		{
			//Just guess I suppose? If there is no font then there's no text to break so it's ok.
			WorldSize = 30.0f;
			InvDefaultSize = 1.0f / 30.0f;
		}
	}

	PrecachePSOs();

	Super::PostLoad();
}

FVertexDeclarationElementList InitDummyVertexDeclarationElementsForText()
{
	FStaticMeshVertexBuffers VertexBuffers;
	VertexBuffers.PositionVertexBuffer.Init(1);
	VertexBuffers.StaticMeshVertexBuffer.Init(1, 1);
	VertexBuffers.ColorVertexBuffer.Init(1);
	
	VertexBuffers.PositionVertexBuffer.VertexPosition(0) = FVector3f(0, 0, 0);
	VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(0, FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1));
	VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(0, 0, FVector2f(0, 0));
	VertexBuffers.ColorVertexBuffer.VertexColor(0) = FColor(1,1,1,1);

	FLocalVertexFactory::FDataType Data;
	VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(nullptr, Data);
	VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(nullptr, Data);
	VertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(nullptr, Data);
	VertexBuffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(nullptr, Data, 0);
	VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(nullptr, Data);
	
	FVertexDeclarationElementList Elements;
	FLocalVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, EVertexInputStreamType::Default, false, Data, Elements);
	return Elements;
}

void UTextRenderComponent::PrecachePSOs()
{
	if (IsComponentPSOPrecachingEnabled() && TextMaterial)
	{
		FPSOPrecacheParams PrecachePSOParams;
		SetupPrecachePSOParams(PrecachePSOParams);

		// Overwrite the cast shadow flag because MeshBatch setup in DrawStaticElements doesn't set the correct CastShadow flag from the proxy or component
		// and leaves the default CastShadow value which is true
		PrecachePSOParams.bCastShadow = true;
	
		FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));
		}
		else
		{
			const static FVertexDeclarationElementList Elements = InitDummyVertexDeclarationElementsForText();
			VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType, Elements));
		}
				
		TArray<FMaterialPSOPrecacheRequestID> MaterialPrecacheRequestIDs;
		TextMaterial->PrecachePSOs(VertexFactoryDataList, PrecachePSOParams, EPSOPrecachePriority::Medium, MaterialPrecacheRequestIDs);
	}
}

void UTextRenderComponent::InitializeMIDCache()
{
	FTextRenderComponentMIDCache::Initialize();
}

void UTextRenderComponent::ShutdownMIDCache()
{
	FTextRenderComponentMIDCache::Shutdown();
}

#undef LOCTEXT_NAMESPACE
