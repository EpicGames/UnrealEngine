// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/VolumetricCloudComponent.h"

#include "Engine/Texture2D.h"
#include "VolumetricCloudProxy.h"
#include "Components/BillboardComponent.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"
#include "RenderingThread.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "SceneInterface.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "LocalVertexFactory.h"
#include "UObject/ICookInfo.h"

#if WITH_EDITOR
#include "MaterialShared.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookEvents.h"
#include "Interfaces/ITargetPlatform.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumetricCloudComponent)

#if WITH_EDITOR
#endif

#define LOCTEXT_NAMESPACE "VolumetricCloudComponent"



/*=============================================================================
	UVolumetricCloudComponent implementation.
=============================================================================*/

UVolumetricCloudComponent::UVolumetricCloudComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LayerBottomAltitude(5.0f)
	, LayerHeight(10.0f)
	, TracingStartMaxDistance(350.0f)
	, TracingStartDistanceFromCamera(0.0f)
	, TracingMaxDistanceMode(EVolumetricCloudTracingMaxDistanceMode::DistanceFromCloudLayerEntryPoint)
	, TracingMaxDistance(50.0f)
	, PlanetRadius(6360.0f)					// Default to earth-like
	, GroundAlbedo(FColor(170, 170, 170))	// 170 => 0.4f linear
	, Material(nullptr)
	, bUsePerSampleAtmosphericLightTransmittance(false)
	, SkyLightCloudBottomOcclusion(0.5f)
	, ViewSampleCountScale(1.0f)
	, ReflectionViewSampleCountScaleValue(1.0f)
	, ReflectionViewSampleCountScale_DEPRECATED(0.15f)		// Roughly equivalent to previous default 1.0f, scaled by OldToNewReflectionViewRaySampleCount
	, ReflectionSampleCountScale_DEPRECATED(1.0f)
	, ShadowViewSampleCountScale(1.0f)
	, ShadowReflectionViewSampleCountScaleValue(1.0f)
	, ShadowReflectionViewSampleCountScale_DEPRECATED(0.3f)	// Roughly equivalent to previous default 1.0f, scaled by OldToNewReflectionShadowRaySampleCount
	, ShadowReflectionSampleCountScale_DEPRECATED(1.0f)
	, ShadowTracingDistance(15.0f)
	, StopTracingTransmittanceThreshold(0.005f)
	, AerialPespectiveRayleighScatteringStartDistance(0.0f)
	, AerialPespectiveRayleighScatteringFadeDistance(0.0f)
	, AerialPespectiveMieScatteringStartDistance(0.0f)
	, AerialPespectiveMieScatteringFadeDistance(0.0f)
	, bHoldout(false)
	, bRenderInMainPass(true)
	, bVisibleInRealTimeSkyCaptures(true)
	, VolumetricCloudSceneProxy(nullptr)
	, MaterialLifetimePtr(nullptr)
{
	// The default material as a soft object pointer, so that the relatively heavy default material with few 3D textures is not always loaded at first.
	Material = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/Engine/EngineSky/VolumetricClouds/m_SimpleVolumetricCloud_Inst.m_SimpleVolumetricCloud_Inst")));
}

UVolumetricCloudComponent::~UVolumetricCloudComponent()
{
}

void UVolumetricCloudComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);
	// If one day we need to look up lightmass built data, lookup it up here using the guid from the correct MapBuildData.

	bool bHidden = false;
#if WITH_EDITORONLY_DATA
	bHidden = GetOwner() ? GetOwner()->bHiddenEdLevel : false;
#endif // WITH_EDITORONLY_DATA
	if (!ShouldComponentAddToScene())
	{
		bHidden = true;
	}

	if (GetVisibleFlag() && !bHidden &&
		ShouldComponentAddToScene() && ShouldRender() && IsRegistered() && (GetOuter() == NULL || !GetOuter()->HasAnyFlags(RF_ClassDefaultObject)))
	{
		// Create the scene proxy.
		VolumetricCloudSceneProxy = new FVolumetricCloudSceneProxy(this);
		GetWorld()->Scene->AddVolumetricCloud(VolumetricCloudSceneProxy);
	}

}

void UVolumetricCloudComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	if (VolumetricCloudSceneProxy)
	{
		GetWorld()->Scene->RemoveVolumetricCloud(VolumetricCloudSceneProxy);

		FVolumetricCloudSceneProxy* SceneProxy = VolumetricCloudSceneProxy;
		ENQUEUE_RENDER_COMMAND(FDestroyVolumetricCloudSceneProxyCommand)(
			[SceneProxy](FRHICommandList& RHICmdList)
		{
			delete SceneProxy;
		});

		VolumetricCloudSceneProxy = nullptr;
	}
}

#if WITH_EDITOR

void UVolumetricCloudComponent::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	Super::OnCookEvent(CookEvent, CookContext);

	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		// Add a runtime dependency on the material asset.
		const FString MaterialLongPackageName = Material.GetLongPackageName();
		CookContext.AddRuntimeDependency(MaterialLongPackageName);
	}
}

void UVolumetricCloudComponent::CheckForErrors()
{
	// Clouds with SkyAtmosphere?
}

void UVolumetricCloudComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

void UVolumetricCloudComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::VolumetricCloudSampleCountUnification && Ar.IsLoading())
	{
		ReflectionViewSampleCountScale_DEPRECATED		= ReflectionSampleCountScale_DEPRECATED		  * UVolumetricCloudComponent::OldToNewReflectionViewRaySampleCount;
		ShadowReflectionViewSampleCountScale_DEPRECATED = ShadowReflectionSampleCountScale_DEPRECATED * UVolumetricCloudComponent::OldToNewReflectionShadowRaySampleCount;
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::VolumetricCloudReflectionSampleCountDefaultUpdate && Ar.IsLoading())
	{
		ReflectionViewSampleCountScaleValue = ReflectionViewSampleCountScale_DEPRECATED;
		ShadowReflectionViewSampleCountScaleValue = ShadowReflectionViewSampleCountScale_DEPRECATED;
	}
}

void UVolumetricCloudComponent::PostLoad()
{
	Super::PostLoad();
	PrecachePSOs();
}

void UVolumetricCloudComponent::OnRegister()
{
	Super::OnRegister();

	// Make sure the assigned TSoftObjectPtr material is loaded by that point so that we can create proxy render side.
	// And it is assigned to a TObjectPtr to make sure it remains loaded and ignored by the GC.
	FCookLoadScope CookLoadScope(ECookLoadType::UsedInGame);
	MaterialLifetimePtr = Material.LoadSynchronous();
}

void UVolumetricCloudComponent::PrecachePSOs()
{
#if UE_WITH_PSO_PRECACHING
	if (!FApp::CanEverRender() || !IsComponentPSOPrecachingEnabled())
	{
		return;
	}

	if (Material && !Material->HasAnyFlags(RF_NeedPostLoad))
	{
		FPSOPrecacheParams PSOPrecacheParams;		
		FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;		
		VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));
		
		// Immediately create at high priority and thus doesn't need boosting anymore
		TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
		Material->PrecachePSOs(VertexFactoryDataList, PSOPrecacheParams, EPSOPrecachePriority::High, MaterialPSOPrecacheRequestIDs);		
	}
#endif
}

#define CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(MemberType, MemberName) void UVolumetricCloudComponent::Set##MemberName(MemberType NewValue)\
{\
	if (AreDynamicDataChangesAllowed() && MemberName != NewValue)\
	{\
		MemberName = NewValue;\
		MarkRenderStateDirty();\
	}\
}\

CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, LayerBottomAltitude);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, LayerHeight);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, TracingStartMaxDistance);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, TracingStartDistanceFromCamera);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, TracingMaxDistance);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, PlanetRadius);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(FColor, GroundAlbedo);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(bool, bUsePerSampleAtmosphericLightTransmittance);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, SkyLightCloudBottomOcclusion);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, ViewSampleCountScale);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, ShadowViewSampleCountScale);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, ShadowTracingDistance);
CLOUD_DECLARE_BLUEPRINT_SETFUNCTION(float, StopTracingTransmittanceThreshold);

void UVolumetricCloudComponent::SetMaterial(UMaterialInterface* NewValue)
{
	if (AreDynamicDataChangesAllowed() && MaterialLifetimePtr.Get() != NewValue)
	{
		Material = NewValue;
		MaterialLifetimePtr = Material.LoadSynchronous();
		MarkRenderStateDirty();

		PrecachePSOs();
	}
}

void UVolumetricCloudComponent::SetHoldout(bool bNewHoldout)
{
	if (bHoldout != bNewHoldout)
	{
		bHoldout = bNewHoldout;
		MarkRenderStateDirty();
	}
}

void UVolumetricCloudComponent::SetRenderInMainPass(bool bValue)
{
	if (bRenderInMainPass != bValue)
	{
		bRenderInMainPass = bValue;
		MarkRenderStateDirty();
	}
}

void UVolumetricCloudComponent::SetVisibleInRealTimeSkyCaptures(bool bValue)
{
	if (bVisibleInRealTimeSkyCaptures != bValue)
	{
		bVisibleInRealTimeSkyCaptures = bValue;
		MarkRenderStateDirty();
	}
}

void UVolumetricCloudComponent::SetReflectionViewSampleCountScale(float NewValue)
{
	if (AreDynamicDataChangesAllowed() && ReflectionViewSampleCountScaleValue != NewValue)
	{
		ReflectionViewSampleCountScaleValue = NewValue;
		MarkRenderStateDirty();
	}
}
void UVolumetricCloudComponent::SetShadowReflectionViewSampleCountScale(float NewValue)
{
	if (AreDynamicDataChangesAllowed() && ShadowReflectionViewSampleCountScaleValue != NewValue)
	{
		ShadowReflectionViewSampleCountScaleValue = NewValue;
		MarkRenderStateDirty();
	}
}

void UVolumetricCloudComponent::SetReflectionSampleCountScale(float NewValue)
{
	if (AreDynamicDataChangesAllowed() && ReflectionViewSampleCountScaleValue != NewValue * UVolumetricCloudComponent::OldToNewReflectionViewRaySampleCount)
	{
		ReflectionViewSampleCountScaleValue = NewValue * UVolumetricCloudComponent::OldToNewReflectionViewRaySampleCount;
		MarkRenderStateDirty();
	}
}
void UVolumetricCloudComponent::SetShadowReflectionSampleCountScale(float NewValue)
{
	if (AreDynamicDataChangesAllowed() && ShadowReflectionViewSampleCountScaleValue != NewValue * UVolumetricCloudComponent::OldToNewReflectionShadowRaySampleCount)
	{
		ShadowReflectionViewSampleCountScaleValue = NewValue * UVolumetricCloudComponent::OldToNewReflectionShadowRaySampleCount;
		MarkRenderStateDirty();
	}
}

/*=============================================================================
	AVolumetricCloud implementation.
=============================================================================*/

#if WITH_EDITOR
#endif

AVolumetricCloud::AVolumetricCloud(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	VolumetricCloudComponent = CreateDefaultSubobject<UVolumetricCloudComponent>(TEXT("VolumetricCloudComponent"));
	RootComponent = VolumetricCloudComponent;

#if WITH_EDITORONLY_DATA

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> VolumetricCloudTextureObject;
			FName ID_VolumetricCloud;
			FText NAME_VolumetricCloud;
			FConstructorStatics()
				: VolumetricCloudTextureObject(TEXT("/Engine/EditorResources/S_VolumetricCloud"))
				, ID_VolumetricCloud(TEXT("Fog"))
				, NAME_VolumetricCloud(NSLOCTEXT("SpriteCategory", "Fog", "Fog"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (GetSpriteComponent())
		{
			GetSpriteComponent()->Sprite = ConstructorStatics.VolumetricCloudTextureObject.Get();
			GetSpriteComponent()->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			GetSpriteComponent()->SpriteInfo.Category = ConstructorStatics.ID_VolumetricCloud;
			GetSpriteComponent()->SpriteInfo.DisplayName = ConstructorStatics.NAME_VolumetricCloud;
			GetSpriteComponent()->SetupAttachment(VolumetricCloudComponent);
		}
	}
#endif // WITH_EDITORONLY_DATA

	PrimaryActorTick.bCanEverTick = true;
	SetHidden(false);
}



#undef LOCTEXT_NAMESPACE



