// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WorldPartitionStreamingSourceComponent.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionStreamingSourceComponent)

UWorldPartitionStreamingSourceComponent::UWorldPartitionStreamingSourceComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, DefaultVisualizerLoadingRange(10000.f)
#endif
	, TargetBehavior(EStreamingSourceTargetBehavior::Include)
	, DebugColor(FColor::MakeRedToGreenColorFromScalar(FRandomStream(GetFName()).GetFraction()))
	, Priority(EStreamingSourcePriority::Normal)
	, TargetState(EStreamingSourceTargetState::Activated)
	, bStreamingSourceEnabled(true)
{
	PrimaryComponentTick.bCanEverTick = false;
}

void UWorldPartitionStreamingSourceComponent::OnRegister()
{
	Super::OnRegister();

	UWorld* World = GetWorld();

#if WITH_EDITOR
	if (!World->IsGameWorld())
	{
		return;
	}
#endif

	UWorldPartitionSubsystem* WorldPartitionSubsystem = GetWorld()->GetSubsystem<UWorldPartitionSubsystem>();
	check(WorldPartitionSubsystem);
	WorldPartitionSubsystem->RegisterStreamingSourceProvider(this);
}

void UWorldPartitionStreamingSourceComponent::OnUnregister()
{
	Super::OnUnregister();

	UWorld* World = GetWorld();

#if WITH_EDITOR
	if (!World->IsGameWorld())
	{
		return;
	}
#endif

	UWorldPartitionSubsystem* WorldPartitionSubsystem = GetWorld()->GetSubsystem<UWorldPartitionSubsystem>();
	check(WorldPartitionSubsystem);
	verify(WorldPartitionSubsystem->UnregisterStreamingSourceProvider(this));
}

void UWorldPartitionStreamingSourceComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

void UWorldPartitionStreamingSourceComponent::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionStreamingSourceComponentTargetDeprecation)
	{
		if (!TargetGrid_DEPRECATED.IsNone())
		{
			TargetGrids.Add(TargetGrid_DEPRECATED);
		}
	}
}

bool UWorldPartitionStreamingSourceComponent::GetStreamingSource(FWorldPartitionStreamingSource& OutStreamingSource) const
{
	if (bStreamingSourceEnabled)
	{
		AActor* Actor = GetOwner();

		OutStreamingSource.Name = *Actor->GetActorNameOrLabel();
		OutStreamingSource.Location = Actor->GetActorLocation();
		OutStreamingSource.Rotation = Actor->GetActorRotation();
		OutStreamingSource.TargetState = TargetState;
		OutStreamingSource.DebugColor = DebugColor;
		OutStreamingSource.TargetBehavior = TargetBehavior;
		OutStreamingSource.Priority = Priority;
						
		if (!Shapes.IsEmpty())
		{
			OutStreamingSource.Shapes.Append(Shapes);
		}

		if (!TargetGrids.IsEmpty())
		{
			OutStreamingSource.TargetGrids.Append(TargetGrids);
		}
		return true;
	}
	return false;
}

bool UWorldPartitionStreamingSourceComponent::IsStreamingCompleted() const
{
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld()))
	{
		return WorldPartitionSubsystem->IsStreamingCompleted(this);
	}
	return true;
}

void UWorldPartitionStreamingSourceComponent::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
#if WITH_EDITOR
	AActor* Actor = GetOwner();
	FStreamingSourceShapeHelper::ForEachShape(DefaultVisualizerLoadingRange, DefaultVisualizerLoadingRange, /*bInProjectIn2D*/ false, Actor->GetActorLocation(), Actor->GetActorRotation(), Shapes, [&PDI](const FSphericalSector& Shape)
	{
		if (Shape.IsSphere())
		{
			DrawWireSphere(PDI, Shape.GetCenter(), FColor::White, Shape.GetRadius(), 32, SDPG_World, 1.0, 0, true);
		}
		else
		{
			TArray<TPair<FVector, FVector>> Lines = Shape.BuildDebugMesh();
			for (const auto& Line : Lines)
			{
				PDI->DrawLine(Line.Key, Line.Value, FColor::White, SDPG_World, 1.0, 0, true);
			};
		}
	});
#endif
}


