// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/CompositeCurveTable.h"
#include "Engine/CurveTable.h"
#include "Misc/MessageDialog.h"
#include "UObject/LinkerLoad.h"

#if WITH_EDITOR
#include "CurveTableEditorUtils.h"
#endif


#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeCurveTable)

#define LOCTEXT_NAMESPACE "CompositeCurveTables"

//////////////////////////////////////////////////////////////////////////
UCompositeCurveTable::UCompositeCurveTable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsLoading = false;
}

void UCompositeCurveTable::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	for (const TObjectPtr<UCurveTable>& ParentTable : ParentTables)
	{
		if (ParentTable)
		{
			OutDeps.Add(ParentTable);
		}
	}
}

void UCompositeCurveTable::PostLoad()
{
	Super::PostLoad();

	bIsLoading = false;
}

void UCompositeCurveTable::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		bIsLoading = true;
	}

	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && GIsTransacting)
	{
		bIsLoading = false;
	}
#endif

	if (bIsLoading)
	{
		for (UCurveTable* ParentTable : ParentTables)
		{
			if (ParentTable)
			{
				ParentTable->ConditionalPreload();
			}
		}

		OnParentTablesUpdated();
	}
}

void UCompositeCurveTable::UpdateCachedRowMap(bool bWarnOnInvalidChildren)
{
	bool bLeaveEmpty = false;
	// Throw up an error message and stop if any loops are found
	if (const UCompositeCurveTable* LoopTable = FindLoops(TArray<const UCompositeCurveTable*>()))
	{
		if (bWarnOnInvalidChildren)
		{
			const FText ErrorMsg = FText::Format(LOCTEXT("FoundLoopError", "Cyclic dependency found. Table {0} depends on itself. Please fix your data"), FText::FromString(LoopTable->GetPathName()));
#if WITH_EDITOR
			if (!bIsLoading)
			{
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMsg);
			}
			else
#endif
			{
				UE_LOGF(LogCurveTable, Warning, "%ls", *ErrorMsg.ToString());
			}
		}
		bLeaveEmpty = true;

		// if the rowmap is empty, stop. We don't need to do the pre and post change since no changes will actually be done
		if (RowMap.Num() == 0)
		{
			return;
		}
	}

#if WITH_EDITOR
	FCurveTableEditorUtils::BroadcastPreChange(this, FCurveTableEditorUtils::ECurveTableChangeInfo::RowList);
#endif
	UCurveTable::EmptyTable();

	if (!bLeaveEmpty)
	{
		CurveTableMode = ECurveTableMode::SimpleCurves;

		// First determine if all our parent tables are simple
		for (const UCurveTable* ParentTable : ParentTables)
		{
			if (ParentTable && ParentTable->GetCurveTableMode() == ECurveTableMode::RichCurves)
			{
				CurveTableMode = ECurveTableMode::RichCurves;
				break;
			}
		}

		// iterate through all of the rows
		for (const UCurveTable* ParentTable : ParentTables)
		{
			if (ParentTable == nullptr)
			{
				continue;
			}

			// Add new rows or overwrite previous rows
			auto AddCurveToMap = [&RowMap=RowMap](FName CurveName, FRealCurve* NewCurve)
			{
				if (FRealCurve** Curve = RowMap.Find(CurveName))
				{
					delete *Curve;
					*Curve = NewCurve;
				}
				else
				{
					RowMap.Add(CurveName, NewCurve);
				}
			};

			UE::TScopeLock ScopeLock(UCurveTable::GetCurveTableChangeCriticalSection());

			// If we are using simple curves we know all our parents are also simple
			if (CurveTableMode == ECurveTableMode::SimpleCurves)
			{
				for (const TPair<FName, FSimpleCurve*>& CurveRow : ParentTable->GetSimpleCurveRowMap())
				{
					FSimpleCurve* NewCurve = new FSimpleCurve();
					FSimpleCurve* InCurve = CurveRow.Value;
					NewCurve->SetKeys(InCurve->GetConstRefOfKeys());
					NewCurve->SetKeyInterpMode(InCurve->GetKeyInterpMode());
					AddCurveToMap(CurveRow.Key, NewCurve);
				}
			}
			// If we are Rich but this Parent is Simple then we need to do a little more conversion work
			else if (ParentTable->GetCurveTableMode() == ECurveTableMode::SimpleCurves)
			{
				for (const TPair<FName, FSimpleCurve*>& CurveRow : ParentTable->GetSimpleCurveRowMap())
				{
					FRichCurve* NewCurve = new FRichCurve();
					FSimpleCurve* InCurve = CurveRow.Value;
					for (const FSimpleCurveKey& CurveKey : InCurve->GetConstRefOfKeys())
					{
						FKeyHandle KeyHandle = NewCurve->AddKey(CurveKey.Time, CurveKey.Value);
						NewCurve->SetKeyInterpMode(KeyHandle, InCurve->GetKeyInterpMode());
					}
					NewCurve->AutoSetTangents();
					AddCurveToMap(CurveRow.Key, NewCurve);
				}
			}
			// Otherwise Rich to Rich is also straightforward
			else
			{
				for (const TPair<FName, FRichCurve*>& CurveRow : ParentTable->GetRichCurveRowMap())
				{
					FRichCurve* NewCurve = new FRichCurve();
					FRichCurve* InCurve = CurveRow.Value;
					NewCurve->SetKeys(InCurve->GetConstRefOfKeys());
					AddCurveToMap(CurveRow.Key, NewCurve);
				}
			}
		}
	}

#if WITH_EDITOR
	FCurveTableEditorUtils::BroadcastPostChange(this, FCurveTableEditorUtils::ECurveTableChangeInfo::RowList);
#endif
}

#if WITH_EDITOR
void UCompositeCurveTable::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static FName Name_ParentTables = GET_MEMBER_NAME_CHECKED(UCompositeCurveTable, ParentTables);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	const FName PropertyName = PropertyThatChanged != nullptr ? PropertyThatChanged->GetFName() : NAME_None;

	if (PropertyName == Name_ParentTables)
	{
		OnParentTablesUpdated(PropertyChangedEvent.ChangeType);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UCompositeCurveTable::PostEditUndo()
{
	OnParentTablesUpdated();

	Super::PostEditUndo();
}
#endif // WITH_EDITOR

void UCompositeCurveTable::OnParentTablesUpdated(EPropertyChangeType::Type ChangeType)
{
	// Prevent recursion when there was a cycle in the parent hierarchy (or during the undo of the action that created the cycle; in that case PostEditUndo will recall OnParentTablesUpdated when the dust has settled)
	if (bUpdatingParentTables)
	{
		return;
	}
	bUpdatingParentTables = true;

	for (UCurveTable* Table : OldParentTables)
	{
		if (Table && ParentTables.Find(Table) == INDEX_NONE)
		{
			Table->OnCurveTableChanged().RemoveAll(this);
		}
	}

	UpdateCachedRowMap(ChangeType == EPropertyChangeType::ValueSet || ChangeType == EPropertyChangeType::Duplicate);

	for (UCurveTable* Table : ParentTables)
	{
		if ((Table != nullptr) && (Table != this) && (OldParentTables.Find(Table) == INDEX_NONE))
		{
			Table->OnCurveTableChanged().AddUObject(this, &UCompositeCurveTable::OnParentTablesUpdated, EPropertyChangeType::Unspecified);
		}
	}

	OldParentTables = ParentTables;

	bUpdatingParentTables = false;
}

void UCompositeCurveTable::EmptyTable()
{
	// clear the parent tables
	ParentTables.Empty();

	Super::EmptyTable();
}

void UCompositeCurveTable::AppendParentTables(const TArray<UCurveTable*>& NewTables)
{
	ParentTables.Append(NewTables);
	OnParentTablesUpdated();
}

const UCompositeCurveTable* UCompositeCurveTable::FindLoops(TArray<const UCompositeCurveTable*> AlreadySeenTables) const
{
	AlreadySeenTables.Add(this);

	for (const TObjectPtr<UCurveTable>& CurveTable : ParentTables)
	{
		// we only care about composite tables since regular tables terminate the chain and can't be in loops
		if (UCompositeCurveTable* CompositeCurveTable = Cast<UCompositeCurveTable>(CurveTable))
		{
			// if we've seen this table before then we have a loop
			for (const UCompositeCurveTable* SeenTable : AlreadySeenTables)
			{
				if (SeenTable == CompositeCurveTable)
				{
					return CompositeCurveTable;
				}
			}

			// recurse
			if (const UCompositeCurveTable* FoundLoop = CompositeCurveTable->FindLoops(AlreadySeenTables))
			{
				return FoundLoop;
			}
		}
	}

	// no loops
	return nullptr;
}

#undef LOCTEXT_NAMESPACE

