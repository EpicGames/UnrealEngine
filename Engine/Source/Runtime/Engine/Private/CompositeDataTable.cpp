// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/CompositeDataTable.h"
#include "Engine/DataTable.h"
#include "UObject/LinkerLoad.h"
#include "Misc/MessageDialog.h"
#if WITH_EDITOR
#include "DataTableEditorUtils.h"
#endif


#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeDataTable)

#define LOCTEXT_NAMESPACE "CompositeDataTables"

#define DATATABLE_CHANGE_SCOPE()	UDataTable::FScopedDataTableChange ActiveScope(this);

UCompositeDataTable::UCompositeDataTable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsLoading = false;
	bShouldNotClearParentTablesOnEmpty = false;
}

void UCompositeDataTable::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	for (UDataTable* Parent : ParentTables)
	{
		if (Parent != nullptr)
		{
			OutDeps.Add(Parent);
		}
	}
}

void UCompositeDataTable::PostLoad()
{
	bIsLoading = false;

	Super::PostLoad();
}

#if WITH_EDITORONLY_DATA
UCompositeDataTable::ERowState UCompositeDataTable::GetRowState(FName RowName) const
{
	const ERowState* RowState = RowSourceMap.Find(RowName);

	return RowState != nullptr ? *RowState : ERowState::Invalid;
}
#endif

void UCompositeDataTable::UpdateCachedRowMap(bool bWarnOnInvalidChildren)
{
	bool bLeaveEmpty = false;
	// Throw up an error message and stop if any loops are found
	if (const UCompositeDataTable* LoopTable = FindLoops(TArray<const UCompositeDataTable*>()))
	{
		if (bWarnOnInvalidChildren)
		{
			const FText ErrorMsg = FText::Format(LOCTEXT("FoundLoopError", "Cyclic dependency found. Table {0} depends on itself. Please fix your data"), FText::FromString(LoopTable->GetPathName()));
#if WITH_EDITOR
			if (!bIsLoading)
			{
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMsg);
			}
			else
#endif
			{
				UE_LOGF(LogDataTable, Warning, "%ls", *ErrorMsg.ToString());
			}
		}
		bLeaveEmpty = true;

		// if the rowmap is empty, stop. We don't need to do the pre and post change since no changes will actually be done
		if (RowMap.Num() == 0)
		{
			return;
		}
	}

	DATATABLE_CHANGE_SCOPE();

#if WITH_EDITOR
	FDataTableEditorUtils::BroadcastPreChange(this, FDataTableEditorUtils::EDataTableChangeInfo::RowList);
#endif
	UDataTable::EmptyTable();

	if (!bLeaveEmpty)
	{
		bool bParentsHaveDifferentRowStruct = false;
		// iterate through all of the rows
		for (const UDataTable* ParentTable : ParentTables)
		{
			if (ParentTable == nullptr)
			{
				continue;
			}
			if (ParentTable->RowStruct != RowStruct)
			{
				if (bWarnOnInvalidChildren)
				{
					bParentsHaveDifferentRowStruct = true;
					FString CompositeRowStructName = RowStruct ? RowStruct->GetName() : "Missing row struct";
					FString ParentRowStructName = ParentTable->RowStruct ? ParentTable->RowStruct->GetName() : "Missing row struct";
					UE_LOGF(LogDataTable, Error, "Composite tables must have the same row struct as their parent tables. Composite Table: %ls, Composite Row Struct: %ls, Parent Table: %ls, Parent Row Struct: %ls.", *GetName(), *CompositeRowStructName, *ParentTable->GetName(), *ParentRowStructName);
				}
				continue;
			}

			// Add new rows or overwrite previous rows
			for (TMap<FName, uint8*>::TConstIterator RowMapIter(ParentTable->GetRowMap().CreateConstIterator()); RowMapIter; ++RowMapIter)
			{
				if (ensure(RowMapIter.Value() != nullptr))
				{
					// UDataTable::AddRow will first remove the row if it already exists so we don't need to do anything special here
					FTableRowBase* TableRowBase = (FTableRowBase*)RowMapIter.Value();
					UDataTable::AddRow(RowMapIter.Key(), *TableRowBase);
				}
			}
		}

		if (bParentsHaveDifferentRowStruct)
		{
			const FText ErrorMsg = FText::Format(LOCTEXT("ParentsIncludesOtherRowStructError", "Composite table '{0}' must have the same row struct as it's parent tables. See output log for list of invalid rows."),
				FText::FromString(GetName()));
#if WITH_EDITOR
			if (!bIsLoading)
			{
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMsg);
			}
			else
#endif
			{
				UE_LOGF(LogDataTable, Warning, "%ls", *ErrorMsg.ToString());
			}
		}
	}
#if WITH_EDITOR
	FDataTableEditorUtils::BroadcastPostChange(this, FDataTableEditorUtils::EDataTableChangeInfo::RowList);
#endif
}

const UCompositeDataTable* UCompositeDataTable::FindLoops(TArray<const UCompositeDataTable*> AlreadySeenTables) const
{
	AlreadySeenTables.Add(this);

	for (const UDataTable* DataTable : ParentTables)
	{
		// we only care about composite tables since regular tables terminate the chain and can't be in loops
		if (const UCompositeDataTable* CompositeDataTable = Cast<UCompositeDataTable>(DataTable))
		{
			// if we've seen this table before then we have a loop
			for (const UCompositeDataTable* SeenTable : AlreadySeenTables)
			{
				if (SeenTable == CompositeDataTable)
				{
					return CompositeDataTable;
				}
			}

			// recurse
			if (const UCompositeDataTable* FoundLoop = CompositeDataTable->FindLoops(AlreadySeenTables))
			{
				return FoundLoop;
			}
		}
	}

	// no loops
	return nullptr;
}

void UCompositeDataTable::EmptyTable()
{
	EmptyCompositeTable(!bIsLoading && !bShouldNotClearParentTablesOnEmpty);
}

void UCompositeDataTable::EmptyCompositeTable(bool bClearParentTables)
{
	if (bClearParentTables)
	{
		ParentTables.Empty();
	}
#if WITH_EDITORONLY_DATA
	RowSourceMap.Empty();
#endif
	UDataTable::EmptyTable();
}

void UCompositeDataTable::RemoveRow(FName RowName)
{
	// do nothing
}

void UCompositeDataTable::AddRow(FName RowName, const FTableRowBase& RowData)
{
	// do nothing
}

void UCompositeDataTable::AddRow(FName RowName, const uint8* RowData, const UScriptStruct* RowType)
{
	// do nothing
}

void UCompositeDataTable::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		bIsLoading = true;
	}

	Super::Serialize(Ar); // When loading, this should load our RowStruct!	

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && GIsTransacting)
	{
		bIsLoading = false;
	}
#endif

	if (bIsLoading)
	{
		for (UDataTable* ParentTable : ParentTables)
		{
			if (ParentTable)
			{
				ParentTable->ConditionalPreload();
			}
		}

		OnParentTablesUpdated();
	}
}

#if WITH_EDITOR
void UCompositeDataTable::CleanBeforeStructChange()
{
	bShouldNotClearParentTablesOnEmpty = true;
	Super::CleanBeforeStructChange();
	bShouldNotClearParentTablesOnEmpty = false;
}

void UCompositeDataTable::RestoreAfterStructChange()
{
	bShouldNotClearParentTablesOnEmpty = true;
	Super::RestoreAfterStructChange();
	bShouldNotClearParentTablesOnEmpty = false;

	UpdateCachedRowMap();
}

void UCompositeDataTable::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static FName Name_ParentTables = GET_MEMBER_NAME_CHECKED(UCompositeDataTable, ParentTables);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	const FName PropertyName = PropertyThatChanged != nullptr ? PropertyThatChanged->GetFName() : NAME_None;

	if (PropertyName == Name_ParentTables)
	{
		OnParentTablesUpdated(PropertyChangedEvent.ChangeType);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UCompositeDataTable::PostEditUndo()
{
	OnParentTablesUpdated(EPropertyChangeType::ValueSet);

	Super::PostEditUndo();
}

#endif // WITH_EDITOR

void UCompositeDataTable::AppendParentTables(const TArray<UDataTable*>& NewTables)
{
	const bool bIsDisregardForGC = GUObjectArray.IsDisregardForGC(this);
	for (UDataTable* DataTable : NewTables)
	{
		if (DataTable)
		{
			if (bIsDisregardForGC)
			{
				DataTable->AddToRoot();
			}

			ParentTables.Add(DataTable);
		}
	}

	OnParentTablesUpdated(EPropertyChangeType::ValueSet);
}

void UCompositeDataTable::RemoveParentTables(const TArray<UDataTable*>& TablesToRemove)
{
	for(UDataTable* DataTable : TablesToRemove)
	{
		if(DataTable)
		{
			DataTable->RemoveFromRoot();
			ParentTables.Remove(DataTable);
		}
	}
	OnParentTablesUpdated(EPropertyChangeType::ValueSet);
}

void UCompositeDataTable::AddParentTable(const TObjectPtr<UDataTable>& TableToAdd)
{
	if (TableToAdd == nullptr)
	{
		return;
	}

	if (GUObjectArray.IsDisregardForGC(this))
	{
		TableToAdd->AddToRoot();
	}

	ParentTables.Add(TableToAdd);
	OnParentTablesUpdated(EPropertyChangeType::ValueSet);
}

void UCompositeDataTable::RemoveParentTable(const TObjectPtr<UDataTable>& TableToRemove)
{
	if(TableToRemove == nullptr)
	{
		return;
	}

	TableToRemove->RemoveFromRoot();
	ParentTables.Remove(TableToRemove);
	OnParentTablesUpdated(EPropertyChangeType::ValueSet);
}

const TArray<UDataTable*>& UCompositeDataTable::GetParentTables() const
{
	return ParentTables;
}


void UCompositeDataTable::OnParentTablesUpdated(EPropertyChangeType::Type ChangeType)
{
	// Prevent recursion when there was a cycle in the parent hierarchy (or during the undo of the action that created the cycle; in that case PostEditUndo will recall OnParentTablesUpdated when the dust has settled)
	if (bUpdatingParentTables)
	{
		return;
	}
	bUpdatingParentTables = true;

	for (UDataTable* Table : OldParentTables)
	{
		if (Table && ParentTables.Find(Table) == INDEX_NONE)
		{
			Table->OnDataTableChanged().RemoveAll(this);
		}
	}

	UpdateCachedRowMap(ChangeType == EPropertyChangeType::ValueSet || ChangeType == EPropertyChangeType::Duplicate);

	for (UDataTable* Table : ParentTables)
	{
		if ((Table != nullptr) && (Table != this) && (OldParentTables.Find(Table) == INDEX_NONE))
		{
			Table->OnDataTableChanged().AddUObject(this, &UCompositeDataTable::OnParentTablesUpdated, EPropertyChangeType::Unspecified);
		}
	}

	OldParentTables = ParentTables;

	bUpdatingParentTables = false;
}

#undef LOCTEXT_NAMESPACE

