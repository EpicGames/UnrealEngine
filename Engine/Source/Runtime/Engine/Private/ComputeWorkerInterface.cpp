// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeWorkerInterface.h"
#include "UObject/NameTypes.h"

void IComputeTaskWorker::SubmitWork(FRDGBuilder& GraphBuilder, FName InExecutionGroupName, ERHIFeatureLevel::Type InFeatureLevel)
{
	FComputeContext Context = FComputeContext
	{
		.GraphBuilder = GraphBuilder,
		.ExecutionGroupName = InExecutionGroupName,
		.FeatureLevel = InFeatureLevel,
		.Scene = nullptr,
		.View = nullptr,
	};

	SubmitWork(Context);
}

FName ComputeTaskExecutionGroup::Immediate("Immediate");
FName ComputeTaskExecutionGroup::EndOfFrameUpdate("EndOfFrameUpdate");
FName ComputeTaskExecutionGroup::BeginInitViews("BeginInitViews");
FName ComputeTaskExecutionGroup::PostTLASBuild("PostTLASBuild");
