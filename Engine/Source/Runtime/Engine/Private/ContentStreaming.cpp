// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ContentStreaming.cpp: Implementation of content streaming classes.
=============================================================================*/

#include "ContentStreaming.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "Misc/ConfigCacheIni.h"
#include "RHI.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Level.h"
#include "RenderingThread.h"
#include "Streaming/StreamingManagerTexture.h"
#include "Animation/AnimationStreaming.h"
#include "AudioStreamingCache.h"
#include "AudioCompressionSettingsUtils.h"
#include "VT/VirtualTextureChunkManager.h"
#include "Rendering/NaniteCoarseMeshStreamingManager.h"
#include "AutoRTFM.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "Streaming/SimpleStreamableAssetManager.h"

#if WITH_EDITOR
#include "AudioDevice.h"
#else
#include "Engine/Engine.h"
#endif

/*-----------------------------------------------------------------------------
	Globals.
-----------------------------------------------------------------------------*/

static TAutoConsoleVariable<int32> CVarMeshStreaming(
	TEXT("r.MeshStreaming"),
	0,
	TEXT("When enabled mesh LODs will stream in based on what is visible on screen."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

/** Collection of views that need to be taken into account for streaming. */
TArray<FStreamingViewInfo> IStreamingManager::CurrentViewInfos;

/** Pending views. Emptied every frame. */
TArray<FStreamingViewInfo> IStreamingManager::PendingViewInfos;

/** Views that stick around for a while. Override views are ignored if no movie is playing. */
TArray<FStreamingViewInfo> IStreamingManager::LastingViewInfos;

/** Collection of view locations that will be added at the next call to AddViewInformation. */
TArray<IStreamingManager::FSecondaryLocation> IStreamingManager::SecondaryLocations;

/** Set when Tick() has been called. The first time a new view is added, it will clear out all old views. */
bool IStreamingManager::bPendingRemoveViews = false;

/**
 * Helper function to flush resource streaming from within Core project.
 */
void FlushResourceStreaming()
{
	RETURN_IF_EXIT_REQUESTED;
	IStreamingManager::Get().BlockTillAllRequestsFinished();
}

/*-----------------------------------------------------------------------------
	Texture tracking.
-----------------------------------------------------------------------------*/

/** Turn on ENABLE_RENDER_ASSET_TRACKING and setup GTrackedTextures to track specific textures/meshes through the streaming system. */
#define ENABLE_RENDER_ASSET_TRACKING !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define ENABLE_RENDER_ASSET_LOGGING 1
#if ENABLE_RENDER_ASSET_TRACKING
struct FTrackedRenderAssetEvent
{
	FTrackedRenderAssetEvent( TCHAR* InAssetName=NULL )
	:	RenderAssetName(InAssetName)
	,	NumResidentMips(0)
	,	NumRequestedMips(0)
	,	WantedMips(0)
	,	Timestamp(0.0f)
	,	BoostFactor(1.0f)
	{
	}
	FTrackedRenderAssetEvent( const FString& InAssetNameString )
	:	RenderAssetName(InAssetNameString)
	,	NumResidentMips(0)
	,	NumRequestedMips(0)
	,	WantedMips(0)
	,	Timestamp(0.0f)
	,	BoostFactor(1.0f)
	{
	}
	/** Partial name of the texture/mesh (not case-sensitive). */
	FString			RenderAssetName;
	/** Number of mip-levels currently in memory. */
	int32			NumResidentMips;
	/** Number of mip-levels requested. */
	int32			NumRequestedMips;
	/** Number of wanted mips. */
	int32			WantedMips;
	/** Timestamp, in seconds from startup. */
	float			Timestamp;
	/** Currently used boost factor for the streaming distance. */
	float			BoostFactor;
};
/** List of textures/meshes to track (using stristr for name comparison). */
TArray<FString> GTrackedRenderAssetNames;
bool GTrackedRenderAssetsInitialized = false;
#define NUM_TRACKEDRENDERASSETEVENTS 512
FTrackedRenderAssetEvent GTrackedRenderAssetEvents[NUM_TRACKEDRENDERASSETEVENTS];
int32 GTrackedRenderAssetEventIndex = -1;
TArray<FTrackedRenderAssetEvent> GTrackedRenderAssets;

/**
 * Initializes the texture/mesh tracking. Called when GTrackedRenderAssetsInitialized is false.
 */
void TrackRenderAssetInit()
{
	if ( GConfig && GConfig->IsReadyForUse() )
	{
		GTrackedRenderAssetNames.Empty();
		GTrackedRenderAssetsInitialized = true;
		GConfig->GetArray(TEXT("RenderAssetTracking"), TEXT("RenderAssetName"), GTrackedRenderAssetNames, GEngineIni);
		TArray<FString> Tmp;
		GConfig->GetArray( TEXT("TextureTracking"), TEXT("TextureName"), Tmp, GEngineIni );
		if (Tmp.Num() > 0)
		{
			GTrackedRenderAssetNames.Append(MoveTemp(Tmp));
			check(!Tmp.Num());
			GConfig->SetArray(TEXT("TextureTracking"), TEXT("TextureName"), Tmp, GEngineIni);
			GConfig->SetArray(TEXT("RenderAssetTracking"), TEXT("RenderAssetName"), GTrackedRenderAssetNames, GEngineIni);
		}
	}
}

/**
 * Adds a (partial) texture/mesh name to track in the streaming system and updates the .ini setting.
 *
 * @param AssetName		Partial name of a new texture/mesh to track (not case-sensitive)
 * @return				true if the name was added
 */
bool TrackRenderAsset( const FString& AssetName )
{
	if ( GConfig && AssetName.Len() > 0 )
	{
		for ( int32 TrackedAssetIndex=0; TrackedAssetIndex < GTrackedRenderAssetNames.Num(); ++TrackedAssetIndex)
		{
			const FString& TrackedAssetName = GTrackedRenderAssetNames[TrackedAssetIndex];
			if ( FCString::Stricmp(*AssetName, *TrackedAssetName) == 0 )
			{
				return false;	
			}
		}

		GTrackedRenderAssetNames.Add( *AssetName );
		GConfig->SetArray( TEXT("RenderAssetTracking"), TEXT("RenderAssetName"), GTrackedRenderAssetNames, GEngineIni );
		return true;
	}
	return false;
}

/**
 * Removes a texture/mesh name from being tracked in the streaming system and updates the .ini setting.
 * The name must match an existing tracking name, but isn't case-sensitive.
 *
 * @param AssetName		Name of a texture/mesh to stop tracking (not case-sensitive)
 * @return				true if the name was removed
 */
bool UntrackRenderAsset( const FString& AssetName )
{
	if ( GConfig && AssetName.Len() > 0 )
	{
		int32 TrackedAssetIndex = 0;
		for ( ; TrackedAssetIndex < GTrackedRenderAssetNames.Num(); ++TrackedAssetIndex)
		{
			const FString& TrackedAssetName = GTrackedRenderAssetNames[TrackedAssetIndex];
			if ( FCString::Stricmp(*AssetName, *TrackedAssetName) == 0 )
			{
				break;
			}
		}
		if ( TrackedAssetIndex < GTrackedRenderAssetNames.Num() )
		{
			GTrackedRenderAssetNames.RemoveAt( TrackedAssetIndex );
			GConfig->SetArray( TEXT("RenderAssetTracking"), TEXT("RenderAssetName"), GTrackedRenderAssetNames, GEngineIni );

			return true;
		}
	}
	return false;
}

/**
 * Lists all currently tracked texture/mesh names in the specified log.
 *
 * @param Ar			Desired output log
 * @param NumAssets		Maximum number of tracked texture/mesh names to output. Outputs all if NumAssets <= 0.
 */
void ListTrackedRenderAssets( FOutputDevice& Ar, int32 NumAssets )
{
	NumAssets = (NumAssets > 0) ? FMath::Min(NumAssets, GTrackedRenderAssetNames.Num()) : GTrackedRenderAssetNames.Num();
	for ( int32 TrackedAssetIndex=0; TrackedAssetIndex < NumAssets; ++TrackedAssetIndex )
	{
		const FString& TrackedAssetName = GTrackedRenderAssetNames[TrackedAssetIndex];
		Ar.Log( TrackedAssetName );
	}
	Ar.Logf( TEXT("%d render assets are being tracked."), NumAssets );
}

/**
 * Checks a texture/mesh and tracks it if its name contains any of the tracked render asset names (GTrackedRenderAssetNames).
 *
 * @param RenderAsset					Texture/mesh to check
 * @param bForceMipLEvelsToBeResident	Whether all mip-levels in the texture/mesh are forced to be resident
 * @param Manager                       can be null
 */
bool TrackRenderAssetEvent(FStreamingRenderAsset* StreamingRenderAsset, UStreamableRenderAsset* RenderAsset, bool bForceMipLevelsToBeResident, const FRenderAssetStreamingManager* Manager)
{
	// Whether the texture/mesh is currently being destroyed
	const bool bIsDestroying = !StreamingRenderAsset;
	const bool bEnableLogging = ENABLE_RENDER_ASSET_LOGGING;

	// Initialize the tracking system, if necessary.
	if ( !GTrackedRenderAssetsInitialized )
	{
		TrackRenderAssetInit();
	}

	int32 NumTrackedAssets = GTrackedRenderAssetNames.Num();
	if (NumTrackedAssets && RenderAsset)
	{
		const FStreamableRenderResourceState ResourceState = RenderAsset->GetStreamableResourceState();

		// See if it matches any of the texture/mesh names that we're tracking.
		FString AssetNameString = RenderAsset->GetFullName();
		const TCHAR* AssetName = *AssetNameString;
		for ( int32 TrackedAssetIndex=0; TrackedAssetIndex < NumTrackedAssets; ++TrackedAssetIndex )
		{
			const FString& TrackedAssetName = GTrackedRenderAssetNames[TrackedAssetIndex];
			if ( FCString::Stristr(AssetName, *TrackedAssetName) != NULL )
			{
				if ( bEnableLogging )
				{
					// Find the last event for this particular texture/mesh.
					FTrackedRenderAssetEvent* LastEvent = NULL;
					for ( int32 LastEventIndex=0; LastEventIndex < GTrackedRenderAssets.Num(); ++LastEventIndex )
					{
						FTrackedRenderAssetEvent* Event = &GTrackedRenderAssets[LastEventIndex];
						if ( FCString::Strcmp(AssetName, *Event->RenderAssetName) == 0 )
						{
							LastEvent = Event;
							break;
						}
					}
					// Didn't find any recorded event?
					if ( LastEvent == NULL )
					{
						int32 NewIndex = GTrackedRenderAssets.Add(FTrackedRenderAssetEvent(AssetNameString));
						LastEvent = &GTrackedRenderAssets[NewIndex];
					}

					int32 WantedMips = ResourceState.NumRequestedLODs;
					float BoostFactor		= 1.0f;
					EStreamableRenderAssetType AssetType = EStreamableRenderAssetType::None;
					if ( StreamingRenderAsset )
					{
						WantedMips			= StreamingRenderAsset->WantedMips;
						BoostFactor			= StreamingRenderAsset->BoostFactor;
						AssetType			= StreamingRenderAsset->RenderAssetType;
					}

					if ( LastEvent->NumResidentMips != ResourceState.NumResidentLODs ||
						 LastEvent->NumRequestedMips != ResourceState.NumRequestedLODs ||
						 LastEvent->WantedMips != WantedMips ||
						 LastEvent->BoostFactor != BoostFactor ||
						 bIsDestroying )
					{
						GTrackedRenderAssetEventIndex	= (GTrackedRenderAssetEventIndex + 1) % NUM_TRACKEDRENDERASSETEVENTS;
						FTrackedRenderAssetEvent& NewEvent	= GTrackedRenderAssetEvents[GTrackedRenderAssetEventIndex];
						NewEvent.RenderAssetName		= LastEvent->RenderAssetName;
						NewEvent.NumResidentMips		= LastEvent->NumResidentMips	= ResourceState.NumResidentLODs;
						NewEvent.NumRequestedMips		= LastEvent->NumRequestedMips	= ResourceState.NumResidentLODs;
						NewEvent.WantedMips				= LastEvent->WantedMips			= WantedMips;
						NewEvent.Timestamp				= LastEvent->Timestamp			= float(FPlatformTime::Seconds() - GStartTime);
						NewEvent.BoostFactor			= LastEvent->BoostFactor		= BoostFactor;
						UE_LOGF(LogContentStreaming, Log, "%ls: \"%ls\", ResidentMips: %d/%d, RequestedMips: %d, WantedMips: %d, Boost: %.1f (%ls)",
							FStreamingRenderAsset::GetStreamingAssetTypeStr(AssetType),
							AssetName, LastEvent->NumResidentMips, ResourceState.MaxNumLODs, bIsDestroying ? 0 : LastEvent->NumRequestedMips, LastEvent->WantedMips, 
							BoostFactor, bIsDestroying ? TEXT("DESTROYED") : TEXT("updated") );
					}
				}
				return true;
			}
		}
	}
	return false;
}

#else
bool TrackRenderAsset(const FString& AssetName)
{
	return false;
}
bool UntrackRenderAsset(const FString& AssetName)
{
	return false;
}
void ListTrackedRenderAssets(FOutputDevice& Ar, int32 NumAssets)
{
}
bool TrackRenderAssetEvent(FStreamingRenderAsset* StreamingRenderAsset, UStreamableRenderAsset* RenderAsset, bool bForceMipLevelsToBeResident, const FRenderAssetStreamingManager* Manager)
{
	return false;
}
#endif

/*-----------------------------------------------------------------------------
	IStreamingManager implementation.
-----------------------------------------------------------------------------*/

static FStreamingManagerCollection* StreamingManagerCollection = nullptr;

FStreamingManagerCollection& IStreamingManager::Get()
{
	if (StreamingManagerCollection == nullptr)
	{
		LLM_SCOPE(ELLMTag::StreamingManager);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		// Since this is a lazily created static global variable we create it
		// in the open.
		UE_AUTORTFM_OPEN
		{
			StreamingManagerCollection = new FStreamingManagerCollection();
		};
	}
	return *StreamingManagerCollection;
}

FStreamingManagerCollection* IStreamingManager::Get_Concurrent()
{
	if (StreamingManagerCollection != (FStreamingManagerCollection*)-1)
	{
		return StreamingManagerCollection;
	}
	else
	{
		return nullptr;
	}
}

void IStreamingManager::Shutdown()
{
	if (StreamingManagerCollection != nullptr)
	{
		delete StreamingManagerCollection;
		StreamingManagerCollection = (FStreamingManagerCollection*)-1;//Force Error if manager used after shutdown
	}
}

bool IStreamingManager::HasShutdown()
{
	return (StreamingManagerCollection == nullptr) || (StreamingManagerCollection == (FStreamingManagerCollection*)-1);
}

/**
 * Adds the passed in view information to the static array.
 *
 * @param ViewInfos				[in/out] Array to add the view to
 * @param ViewOrigin			View origin
 * @param ScreenSize			Screen size
 * @param FOVScreenSize			Screen size taking FOV into account
 * @param BoostFactor			A factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa.
 * @param bOverrideLocation		Whether this is an override location, which forces the streaming system to ignore all other regular locations
 * @param Duration				How long the streaming system should keep checking this location (in seconds). 0 means just for the next Tick.
 * @param InActorToBoost		Optional pointer to an actor who's textures should have their streaming priority boosted
 */
void IStreamingManager::AddViewInfoToArray( TArray<FStreamingViewInfo> &ViewInfos, const FVector& ViewOrigin, float ScreenSize, float FOVScreenSize, float BoostFactor, bool bOverrideLocation, float Duration, TWeakObjectPtr<AActor> InActorToBoost, TWeakObjectPtr<UWorld> InWorld)
{
	// Check for duplicates and existing overrides.
	bool bShouldAddView = true;
	for ( int32 ViewIndex=0; ViewIndex < ViewInfos.Num(); ++ViewIndex )
	{
		FStreamingViewInfo& ViewInfo = ViewInfos[ ViewIndex ];
		if ( ViewOrigin.Equals( ViewInfo.ViewOrigin, 0.5f ) &&
			FMath::IsNearlyEqual( ScreenSize, ViewInfo.ScreenSize ) &&
			FMath::IsNearlyEqual( FOVScreenSize, ViewInfo.FOVScreenSize ) &&
			ViewInfo.bOverrideLocation == bOverrideLocation )
		{
			// Update duration
			ViewInfo.Duration = Duration;
			// Overwrite BoostFactor if it isn't default 1.0
			ViewInfo.BoostFactor = FMath::IsNearlyEqual(BoostFactor, 1.0f) ? ViewInfo.BoostFactor : BoostFactor;
			bShouldAddView = false;
		}
	}

	if ( bShouldAddView )
	{
		new (ViewInfos) FStreamingViewInfo( ViewOrigin, ScreenSize, FOVScreenSize, BoostFactor, bOverrideLocation, Duration, InActorToBoost, InWorld );
	}
}

/**
 * Remove view infos with the same location from the given array.
 *
 * @param ViewInfos				[in/out] Array to remove the view from
 * @param ViewOrigin			View origin
 */
void IStreamingManager::RemoveViewInfoFromArray( TArray<FStreamingViewInfo> &ViewInfos, const FVector& ViewOrigin )
{
	for ( int32 ViewIndex=0; ViewIndex < ViewInfos.Num(); ++ViewIndex )
	{
		FStreamingViewInfo& ViewInfo = ViewInfos[ ViewIndex ];
		if ( ViewOrigin.Equals( ViewInfo.ViewOrigin, 0.5f ) )
		{
			ViewInfos.RemoveAtSwap( ViewIndex-- );
		}
	}
}

#if STREAMING_LOG_VIEWCHANGES
TArray<FStreamingViewInfo> GPrevViewLocations;
#endif

/**
 * Sets up the CurrentViewInfos array based on PendingViewInfos, LastingViewInfos and SecondaryLocations.
 * Removes out-dated LastingViewInfos.
 *
 * @param DeltaTime		Time since last call in seconds
 */
void IStreamingManager::SetupViewInfos( float DeltaTime )
{
	// Reset CurrentViewInfos
	CurrentViewInfos.Empty( PendingViewInfos.Num() + LastingViewInfos.Num() + SecondaryLocations.Num() );

	bool bHaveMultiplePlayerViews = (PendingViewInfos.Num() > 1) ? true : false;

	// Add the secondary locations.
	float ScreenSize = 1280.0f;
	float FOVScreenSize = ScreenSize / FMath::Tan( 80.0f * float(UE_PI) / 360.0f );
	if ( PendingViewInfos.Num() > 0 )
	{
		ScreenSize = PendingViewInfos[0].ScreenSize;
		FOVScreenSize = PendingViewInfos[0].FOVScreenSize;
	}
	else if ( LastingViewInfos.Num() > 0 )
	{
		ScreenSize = LastingViewInfos[0].ScreenSize;
		FOVScreenSize = LastingViewInfos[0].FOVScreenSize;
	}

	// Add them to the appropriate array (pending views or lasting views).
	{
		// Disable this flag as it could be used in AddViewInformation to empty SecondaryLocation.
		const bool bPendingRemoveViewsBackup = bPendingRemoveViews;
		bPendingRemoveViews = false;

		for ( int32 SecondaryLocationIndex=0; SecondaryLocationIndex < SecondaryLocations.Num(); SecondaryLocationIndex++ )
		{
			const FSecondaryLocation& SecondaryLocation = SecondaryLocations[ SecondaryLocationIndex ];
			AddViewInformation( SecondaryLocation.Location, ScreenSize, FOVScreenSize, SecondaryLocation.BoostFactor, SecondaryLocation.bOverrideLocation, SecondaryLocation.Duration );
		}

		bPendingRemoveViews = bPendingRemoveViewsBackup;
	}

	// Apply a split-screen factor if we have multiple players on the same machine, and they currently have individual views.
	float SplitScreenFactor = 1.0f;
	
	if ( bHaveMultiplePlayerViews && GEngine->HasMultipleLocalPlayers(NULL) )
	{
		SplitScreenFactor = 0.75f;
	}

	// Should we use override views this frame? (If we have both a fullscreen movie and an override view.)
	bool bUseOverrideViews = false;
	bool bIsMoviePlaying = false;//GetMoviePlayer()->IsMovieCurrentlyPlaying();
	if ( bIsMoviePlaying )
	{
		// Check if we have any override views.
		for ( int32 ViewIndex=0; !bUseOverrideViews && ViewIndex < LastingViewInfos.Num(); ++ViewIndex )
		{
			const FStreamingViewInfo& ViewInfo = LastingViewInfos[ ViewIndex ];
			if ( ViewInfo.bOverrideLocation )
			{
				bUseOverrideViews = true;
			}
		}
		for ( int32 ViewIndex=0; !bUseOverrideViews && ViewIndex < PendingViewInfos.Num(); ++ViewIndex )
		{
			const FStreamingViewInfo& ViewInfo = PendingViewInfos[ ViewIndex ];
			if ( ViewInfo.bOverrideLocation )
			{
				bUseOverrideViews = true;
			}
		}
	}

	// Add the lasting views.
	for ( int32 ViewIndex=0; ViewIndex < LastingViewInfos.Num(); ++ViewIndex )
	{
		const FStreamingViewInfo& ViewInfo = LastingViewInfos[ ViewIndex ];
		if ( bUseOverrideViews == ViewInfo.bOverrideLocation )
		{
			AddViewInfoToArray( CurrentViewInfos, ViewInfo.ViewOrigin, ViewInfo.ScreenSize * SplitScreenFactor, ViewInfo.FOVScreenSize, ViewInfo.BoostFactor, ViewInfo.bOverrideLocation, ViewInfo.Duration, ViewInfo.ActorToBoost, ViewInfo.World );
		}
	}

	// Add the regular views.
	for ( int32 ViewIndex=0; ViewIndex < PendingViewInfos.Num(); ++ViewIndex )
	{
		const FStreamingViewInfo& ViewInfo = PendingViewInfos[ ViewIndex ];
		if ( bUseOverrideViews == ViewInfo.bOverrideLocation )
		{
			AddViewInfoToArray( CurrentViewInfos, ViewInfo.ViewOrigin, ViewInfo.ScreenSize * SplitScreenFactor, ViewInfo.FOVScreenSize, ViewInfo.BoostFactor, ViewInfo.bOverrideLocation, ViewInfo.Duration, ViewInfo.ActorToBoost, ViewInfo.World );
		}
	}

	// Update duration for the lasting views, removing out-dated ones.
	for ( int32 ViewIndex=0; ViewIndex < LastingViewInfos.Num(); ++ViewIndex )
	{
		FStreamingViewInfo& ViewInfo = LastingViewInfos[ ViewIndex ];
		ViewInfo.Duration -= DeltaTime;

		// Remove old override locations.
		if ( ViewInfo.Duration <= 0.0f )
		{
			LastingViewInfos.RemoveAtSwap( ViewIndex-- );
		}
	}

#if STREAMING_LOG_VIEWCHANGES
	{
		// Check if we're adding any new locations.
		for ( int32 ViewIndex = 0; ViewIndex < CurrentViewInfos.Num(); ++ViewIndex )
		{
			FStreamingViewInfo& ViewInfo = CurrentViewInfos[ViewIndex];
			bool bFound = false;
			for ( int32 PrevView=0; PrevView < GPrevViewLocations.Num(); ++PrevView )
			{
				if ( (ViewInfo.ViewOrigin - GPrevViewLocations[PrevView].ViewOrigin).SizeSquared() < 10000.0f )
				{
					bFound = true;
					break;
				}
			}
			if ( !bFound )
			{
				UE_LOGF(LogContentStreaming, Log, "Adding location: X=%.1f, Y=%.1f, Z=%.1f (override=%d, boost=%.1f)", ViewInfo.ViewOrigin.X, ViewInfo.ViewOrigin.Y, ViewInfo.ViewOrigin.Z, ViewInfo.bOverrideLocation, ViewInfo.BoostFactor );
			}
		}

		// Check if we're removing any locations.
		for ( int32 PrevView=0; PrevView < GPrevViewLocations.Num(); ++PrevView )
		{
			bool bFound = false;
			for ( int32 ViewIndex = 0; ViewIndex < CurrentViewInfos.Num(); ++ViewIndex )
			{
				FStreamingViewInfo& ViewInfo = CurrentViewInfos[ViewIndex];
				if ( (ViewInfo.ViewOrigin - GPrevViewLocations[PrevView].ViewOrigin).SizeSquared() < 10000.0f )
				{
					bFound = true;
					break;
				}
			}
			if ( !bFound )
			{
				FStreamingViewInfo& PrevViewInfo = GPrevViewLocations[PrevView];
				UE_LOGF(LogContentStreaming, Log, "Removing location: X=%.1f, Y=%.1f, Z=%.1f (override=%d, boost=%.1f)", PrevViewInfo.ViewOrigin.X, PrevViewInfo.ViewOrigin.Y, PrevViewInfo.ViewOrigin.Z, PrevViewInfo.bOverrideLocation, PrevViewInfo.BoostFactor );
			}
		}

		// Save the locations.
		GPrevViewLocations.Empty(CurrentViewInfos.Num());
		for ( int32 ViewIndex = 0; ViewIndex < CurrentViewInfos.Num(); ++ViewIndex )
		{
			FStreamingViewInfo& ViewInfo = CurrentViewInfos[ ViewIndex ];
			GPrevViewLocations.Add( ViewInfo );
		}
	}
#endif
}

/**
 * Adds the passed in view information to the static array.
 *
 * @param ViewOrigin			View origin
 * @param ScreenSize			Screen size
 * @param FOVScreenSize			Screen size taking FOV into account
 * @param BoostFactor			A factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa.
 * @param bOverrideLocation		Whether this is an override location, which forces the streaming system to ignore all other regular locations
 * @param Duration				How long the streaming system should keep checking this location (in seconds). 0 means just for the next Tick.
 * @param InActorToBoost		Optional pointer to an actor who's textures should have their streaming priority boosted
 */
void IStreamingManager::AddViewInformation( const FVector& ViewOrigin, float ScreenSize, float FOVScreenSize, float BoostFactor/*=1.0f*/, bool bOverrideLocation/*=false*/, float Duration/*=0.0f*/, TWeakObjectPtr<AActor> InActorToBoost /*=NULL*/, TWeakObjectPtr<UWorld> InWorld /*= NULL*/)
{
	// Is this a reasonable location?
	if ( FMath::Abs(ViewOrigin.X) < (1.0e+20f) && FMath::Abs(ViewOrigin.Y) < (1.0e+20f) && FMath::Abs(ViewOrigin.Z) < (1.0e+20f) )
	{
		const float MinBoost = CVarStreamingMinBoost.GetValueOnGameThread();
		const float BoostScale = FMath::Max(CVarStreamingBoost.GetValueOnGameThread(),MinBoost);
		BoostFactor *= BoostScale;

		if ( bPendingRemoveViews )
		{
			bPendingRemoveViews = false;

			// Remove out-dated override views and empty the PendingViewInfos/SecondaryLocation arrays to be populated again during next frame.
			RemoveStreamingViews( RemoveStreamingViews_Normal );
		}

		// Remove a lasting location if we're given the same location again but with 0 duration.
		if ( Duration <= 0.0f )
		{
			RemoveViewInfoFromArray( LastingViewInfos, ViewOrigin );
		}

		// Should we remember this location for a while?
		if ( Duration > 0.0f )
		{
			AddViewInfoToArray( LastingViewInfos, ViewOrigin, ScreenSize, FOVScreenSize, BoostFactor, bOverrideLocation, Duration, InActorToBoost, InWorld );
		}
		else
		{
			AddViewInfoToArray( PendingViewInfos, ViewOrigin, ScreenSize, FOVScreenSize, BoostFactor, bOverrideLocation, 0.0f, InActorToBoost, InWorld );
		}
	}
	else
	{
		int SetDebugBreakpointHere = 0;
	}
}

/**
 * Queue up view locations to the streaming system. These locations will be added properly at the next call to AddViewInformation,
 * re-using the screensize and FOV settings.
 *
 * @param Location				World-space view origin
 * @param BoostFactor			A factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa.
 * @param bOverrideLocation		Whether this is an override location, which forces the streaming system to ignore all other locations
 * @param Duration				How long the streaming system should keep checking this location (in seconds). 0 means just for the next Tick.
 */
void IStreamingManager::AddViewLocation( const FVector& Location, float BoostFactor/*=1.0f*/, bool bOverrideLocation/*=false*/, float Duration/*=0.0f*/ )
{
	const float MinBoost = CVarStreamingMinBoost.GetValueOnGameThread();
	const float BoostScale = FMath::Max(CVarStreamingBoost.GetValueOnGameThread(),MinBoost);
	BoostFactor *= BoostScale;

	if ( bPendingRemoveViews )
	{
		bPendingRemoveViews = false;

		// Remove out-dated override views and empty the PendingViewInfos/SecondaryLocation arrays to be populated again during next frame.
		RemoveStreamingViews( RemoveStreamingViews_Normal );
	}

	new (SecondaryLocations) FSecondaryLocation(Location, BoostFactor, bOverrideLocation, Duration );
}

/**
 * Removes streaming views from the streaming manager. This is also called by Tick().
 *
 * @param RemovalType	What types of views to remove (all or just the normal views)
 */
void IStreamingManager::RemoveStreamingViews( ERemoveStreamingViews RemovalType )
{
	PendingViewInfos.Empty();
	SecondaryLocations.Empty();
	if ( RemovalType == RemoveStreamingViews_All )
	{
		LastingViewInfos.Empty();
	}
}

/**
 * Calls UpdateResourceStreaming(), and does per-frame cleaning. Call once per frame.
 *
 * @param DeltaTime				Time since last call in seconds
 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
 */
void IStreamingManager::Tick( float DeltaTime, bool bProcessEverything/*=false*/ )
{
	LLM_SCOPE(ELLMTag::StreamingManager);

	UpdateResourceStreaming( DeltaTime, bProcessEverything );

	// Trigger a call to RemoveStreamingViews( RemoveStreamingViews_Normal ) next time a view is added.
	bPendingRemoveViews = true;
}

int32 IStreamingManager::StreamAllResources(float TimeLimit)
{
	return 0;
}

/*-----------------------------------------------------------------------------
	IRenderAssetStreamingManager implementation.
-----------------------------------------------------------------------------*/

void IRenderAssetStreamingManager::UpdateIndividualTexture(UTexture2D* Texture)
{
	UpdateIndividualRenderAsset(Texture);
}

bool IRenderAssetStreamingManager::StreamOutTextureData(int64 RequiredMemorySize)
{
	return StreamOutRenderAssetData(RequiredMemorySize);
}

void IRenderAssetStreamingManager::AddStreamingTexture(UTexture2D* Texture)
{
	AddStreamingRenderAsset(Texture);
}

void IRenderAssetStreamingManager::RemoveStreamingTexture(UTexture2D* Texture)
{
	RemoveStreamingRenderAsset(Texture);
}

void IRenderAssetStreamingManager::PauseTextureStreaming(bool bInShouldPause)
{
	PauseRenderAssetStreaming(bInShouldPause);
}

/*-----------------------------------------------------------------------------
	FStreamingManagerCollection implementation.
-----------------------------------------------------------------------------*/

FStreamingManagerCollection::FStreamingManagerCollection()
	: NumIterations(1)
	, DisableResourceStreamingCount(0)
	, LoadMapTimeLimit(5.0f)
	, RenderAssetStreamingManager(nullptr)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, NaniteCoarseMeshStreamingManager(nullptr)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
#if PLATFORM_SUPPORTS_TEXTURE_STREAMING
	// Disable texture streaming if that was requested (needs to happen before the call to ProcessNewlyLoadedUObjects, as that can load textures)
	if( FParse::Param( FCommandLine::Get(), TEXT( "NoTextureStreaming" ) ) )
	{
		CVarSetTextureStreaming.AsVariable()->Set(0, ECVF_SetByCommandline);
	}
#endif

	AddOrRemoveTextureStreamingManagerIfNeeded(true);

	if (FApp::CanEverRenderAudio())
	{
		FCachedAudioStreamingManagerParams Params = FPlatformCompressionUtilities::BuildCachedStreamingManagerParams();
		AudioStreamingManager = new FCachedAudioStreamingManager(Params);
	}
	else
	{
		// cannot render any audio, but code still expects this class to exist.
		AudioStreamingManager = new FDummyAudioStreamingManager();
	}
	
	AddStreamingManager( AudioStreamingManager );

	AnimationStreamingManager = new FAnimationStreamingManager();
	AddStreamingManager(AnimationStreamingManager);

	VirtualTextureStreamingManager = new FVirtualTextureChunkStreamingManager();
	AddStreamingManager(VirtualTextureStreamingManager);

	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		FSimpleStreamableAssetManager::Init();
	}
}

FStreamingManagerCollection::~FStreamingManagerCollection()
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		FSimpleStreamableAssetManager::Shutdown();
	}

	RemoveStreamingManager(VirtualTextureStreamingManager);
	delete VirtualTextureStreamingManager;
	VirtualTextureStreamingManager = nullptr;

	RemoveStreamingManager(AnimationStreamingManager);
	delete AnimationStreamingManager;
	AnimationStreamingManager = nullptr;

	RemoveStreamingManager(AudioStreamingManager);
	delete AudioStreamingManager;
	AudioStreamingManager = nullptr;

	RemoveStreamingManager(RenderAssetStreamingManager);
	delete RenderAssetStreamingManager;
	RenderAssetStreamingManager = nullptr;

	UE_CLOGF(StreamingManagers.Num() > 0, LogContentStreaming, Display, "There are %d unreleased StreamingManagers", StreamingManagers.Num());
}

/**
 * Sets the number of iterations to use for the next time UpdateResourceStreaming is being called. This 
 * is being reset to 1 afterwards.
 *
 * @param InNumIterations	Number of iterations to perform the next time UpdateResourceStreaming is being called.
 */
void FStreamingManagerCollection::SetNumIterationsForNextFrame( int32 InNumIterations )
{
	NumIterations = InNumIterations;
}

/**
 * Calls UpdateResourceStreaming(), and does per-frame cleaning. Call once per frame.
 *
 * @param DeltaTime				Time since last call in seconds
 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
 */
void FStreamingManagerCollection::Tick( float DeltaTime, bool bProcessEverything )
{
	LLM_SCOPE(ELLMTag::StreamingManager);

	AddOrRemoveTextureStreamingManagerIfNeeded();

	IStreamingManager::Tick(DeltaTime, bProcessEverything);
}

/**
 * Updates streaming, taking into account all current view infos. Can be called multiple times per frame.
 *
 * @param DeltaTime				Time since last call in seconds
 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
 */

void FStreamingManagerCollection::UpdateResourceStreaming( float DeltaTime, bool bProcessEverything/*=false*/ )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamingManagerCollection::UpdateResourceStreaming);
	SetupViewInfos( DeltaTime );

	// only allow this if its not disabled
	if (DisableResourceStreamingCount == 0)
	{
		for( int32 Iteration=0; Iteration<NumIterations; Iteration++ )
		{
			// Flush rendering commands in the case of multiple iterations to sync up resource streaming
			// with the GPU/ rendering thread.
			if( Iteration > 0 )
			{
				FlushRenderingCommands();
			}

			// Route to streaming managers.
			for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
			{
				IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
				StreamingManager->UpdateResourceStreaming( DeltaTime, bProcessEverything );
			}
		}

		// Reset number of iterations to 1 for next frame.
		NumIterations = 1;
	}
}

/**
 * Streams in/out all resources that wants to and blocks until it's done.
 *
 * @param TimeLimit					Maximum number of seconds to wait for streaming I/O. If zero, uses .ini setting
 * @return							Number of streaming requests still in flight, if the time limit was reached before they were finished.
 */
int32 FStreamingManagerCollection::StreamAllResources( float TimeLimit/*=0.0f*/ )
{
	FlushRenderingCommands();

	if (FMath::IsNearlyZero(TimeLimit))
	{
		TimeLimit = LoadMapTimeLimit;
	}

	// Update resource streaming, making sure we process all textures.
	UpdateResourceStreaming(0, true);

	// Block till requests are finished, or time limit was reached.
	int32 NumPendingRequests = BlockTillAllRequestsFinished(TimeLimit, true);

	return NumPendingRequests;
}

/**
 * Blocks till all pending requests are fulfilled.
 *
 * @param TimeLimit		Optional time limit for processing, in seconds. Specifying 0 means infinite time limit.
 * @param bLogResults	Whether to dump the results to the log.
 * @return				Number of streaming requests still in flight, if the time limit was reached before they were finished.
 */
int32 FStreamingManagerCollection::BlockTillAllRequestsFinished( float TimeLimit /*= 0.0f*/, bool bLogResults /*= false*/ )
{
	int32 NumPendingRequests = 0;

	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		NumPendingRequests += StreamingManager->BlockTillAllRequestsFinished( TimeLimit, bLogResults );
	}

	return NumPendingRequests;
}

/** Returns the number of resources that currently wants to be streamed in. */
int32 FStreamingManagerCollection::GetNumWantingResources() const
{
	int32 NumResourcesWantingStreaming = 0;

	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		NumResourcesWantingStreaming += StreamingManager->GetNumWantingResources();
	}

	return NumResourcesWantingStreaming;
}

/**
 * Returns the current ID for GetNumWantingResources().
 * The ID is bumped every time NumWantingResources is updated by the streaming system (every few frames).
 * Can be used to verify that any changes have been fully examined, by comparing current ID with
 * what it was when the changes were made.
 */
int32 FStreamingManagerCollection::GetNumWantingResourcesID() const
{
	int32 NumWantingResourcesStreamingCounter = MAX_int32;

	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		NumWantingResourcesStreamingCounter = FMath::Min( NumWantingResourcesStreamingCounter, StreamingManager->GetNumWantingResourcesID() );
	}

	return NumWantingResourcesStreamingCounter;
}

/**
 * Cancels the timed Forced resources (i.e used the Kismet action "Stream In Textures").
 */
void FStreamingManagerCollection::CancelForcedResources()
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->CancelForcedResources();
	}
}

/**
 * Notifies managers of "level" change.
 */
void FStreamingManagerCollection::NotifyLevelChange()
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->NotifyLevelChange();
	}
}

bool FStreamingManagerCollection::IsStreamingEnabled() const
{
	return DisableResourceStreamingCount == 0;
}

bool FStreamingManagerCollection::IsTextureStreamingEnabled() const
{
	return IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::Texture);
}

bool FStreamingManagerCollection::IsRenderAssetStreamingEnabled(EStreamableRenderAssetType FilteredAssetType) const
{
	if (RenderAssetStreamingManager)
	{
		switch (FilteredAssetType)
		{
		case EStreamableRenderAssetType::None:
			return true;
		case EStreamableRenderAssetType::Texture:
			return FPlatformProperties::SupportsTextureStreaming();
		case EStreamableRenderAssetType::StaticMesh:
		case EStreamableRenderAssetType::SkeletalMesh:
			return FPlatformProperties::SupportsMeshLODStreaming() && CVarMeshStreaming.GetValueOnAnyThread() != 0;
		default:
			break;
		}
	}
	return false;
}

IRenderAssetStreamingManager& FStreamingManagerCollection::GetTextureStreamingManager() const
{
	return GetRenderAssetStreamingManager();
}

IRenderAssetStreamingManager& FStreamingManagerCollection::GetRenderAssetStreamingManager() const
{
	check(RenderAssetStreamingManager != 0);
	return *RenderAssetStreamingManager;
}

IAudioStreamingManager& FStreamingManagerCollection::GetAudioStreamingManager() const
{
#if WITH_EDITOR
	FScopeLock ScopeLock(&AudioStreamingManagerCriticalSection);
#endif

	check(AudioStreamingManager);
	return *AudioStreamingManager;
}

IAnimationStreamingManager& FStreamingManagerCollection::GetAnimationStreamingManager() const
{
	check(AnimationStreamingManager);
	return *AnimationStreamingManager;
}

FVirtualTextureChunkStreamingManager& FStreamingManagerCollection::GetVirtualTextureStreamingManager() const
{
	check(VirtualTextureStreamingManager);
	return *VirtualTextureStreamingManager;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
Nanite::FCoarseMeshStreamingManager* FStreamingManagerCollection::GetNaniteCoarseMeshStreamingManager() const
{
	return NaniteCoarseMeshStreamingManager;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Don't stream world resources for the next NumFrames. */
void FStreamingManagerCollection::SetDisregardWorldResourcesForFrames(int32 NumFrames )
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->SetDisregardWorldResourcesForFrames(NumFrames);
	}
}

/**
 * Adds a streaming manager to the array of managers to route function calls to.
 *
 * @param StreamingManager	Streaming manager to add
 */
void FStreamingManagerCollection::AddStreamingManager( IStreamingManager* StreamingManager )
{
	StreamingManagers.Add( StreamingManager );
}

/**
 * Removes a streaming manager from the array of managers to route function calls to.
 *
 * @param StreamingManager	Streaming manager to remove
 */
void FStreamingManagerCollection::RemoveStreamingManager( IStreamingManager* StreamingManager )
{
	StreamingManagers.Remove( StreamingManager );
}

/**
 * Disables resource streaming. Enable with EnableResourceStreaming. Disable/enable can be called multiple times nested
 */
void FStreamingManagerCollection::DisableResourceStreaming()
{
	// push on a disable
	FPlatformAtomics::InterlockedIncrement(&DisableResourceStreamingCount);
}

/**
 * Enables resource streaming, previously disabled with enableResourceStreaming. Disable/enable can be called multiple times nested
 * (this will only actually enable when all disables are matched with enables)
 */
void FStreamingManagerCollection::EnableResourceStreaming()
{
	// pop off a disable
	FPlatformAtomics::InterlockedDecrement(&DisableResourceStreamingCount);

	checkf(DisableResourceStreamingCount >= 0, TEXT("Mismatched number of calls to FStreamingManagerCollection::DisableResourceStreaming/EnableResourceStreaming"));
}

/**
 * Allows the streaming manager to process exec commands.
 *
 * @param Cmd	Exec command
 * @param Ar	Output device for feedback
 * @return		true if the command was handled
 */
bool FStreamingManagerCollection::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) 
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		if ( StreamingManager->Exec( InWorld, Cmd, Ar ) )
		{
			return true;
		}
	}
	return false;
}

/** Adds a ULevel to the streaming manager. */
void FStreamingManagerCollection::AddLevel( ULevel* Level )
{
#if STREAMING_LOG_LEVELS
	UE_LOGF(LogContentStreaming, Log, "FStreamingManagerCollection::AddLevel(\"%ls\")", *Level->GetOutermost()->GetName());
#endif

	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->AddLevel( Level );
	}
}

/** Removes a ULevel from the streaming manager. */
void FStreamingManagerCollection::RemoveLevel( ULevel* Level )
{
#if STREAMING_LOG_LEVELS
	UE_LOGF(LogContentStreaming, Log, "FStreamingManagerCollection::RemoveLevel(\"%ls\")", *Level->GetOutermost()->GetName());
#endif

	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->RemoveLevel( Level );
	}
}

void FStreamingManagerCollection::NotifyLevelOffset(ULevel* Level, const FVector& Offset)
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->NotifyLevelOffset( Level, Offset );
	}
}

/** Called when a spawned actor is destroyed. */
void FStreamingManagerCollection::NotifyActorDestroyed( AActor* Actor )
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->NotifyActorDestroyed( Actor );
	}
}

/** Called when a primitive is detached from an actor or another component. */
void FStreamingManagerCollection::NotifyPrimitiveDetached( const IPrimitiveComponent* Primitive )
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->NotifyPrimitiveDetached( Primitive );
	}
}

/** Called when a primitive streaming data needs to be updated. */
void FStreamingManagerCollection::NotifyPrimitiveUpdated( const IPrimitiveComponent* Primitive )
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->NotifyPrimitiveUpdated( Primitive );
	}
}

/**  Called when a primitive streaming data needs to be updated in the last stage of the frame. */
void FStreamingManagerCollection::NotifyPrimitiveUpdated_Concurrent( const IPrimitiveComponent* Primitive )
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->NotifyPrimitiveUpdated_Concurrent( Primitive );
	}
}

void FStreamingManagerCollection::PropagateLightingScenarioChange()
{
	// Route to streaming managers.
	for( int32 ManagerIndex=0; ManagerIndex<StreamingManagers.Num(); ManagerIndex++ )
	{
		IStreamingManager* StreamingManager = StreamingManagers[ManagerIndex];
		StreamingManager->PropagateLightingScenarioChange();
	}
}

#if WITH_EDITOR

void FStreamingManagerCollection::OnAudioStreamingParamsChanged()
{
	// Before we swap out the audio streaming manager, we'll need to stop all sounds running on all audio devices:
	FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
	TArray<FAudioDevice*> AudioDevices = DeviceManager->GetAudioDevices();
	for (FAudioDevice* AudioDevice : AudioDevices)
	{
		if (AudioDevice)
		{
			AudioDevice->StopAllSounds(true);
		}
	}

	FScopeLock ScopeLock(&AudioStreamingManagerCriticalSection);

	RemoveStreamingManager(AudioStreamingManager);

	delete AudioStreamingManager;
	AudioStreamingManager = nullptr;

	// Lastly, make sure we have the most up to date cook overrides cached:
	FPlatformCompressionUtilities::RecacheCookOverrides();

	// Finally, reinitialize the streaming manager.
	FCachedAudioStreamingManagerParams Params = FPlatformCompressionUtilities::BuildCachedStreamingManagerParams();
	AddStreamingManager(new FCachedAudioStreamingManager(Params));
}
#endif

void FStreamingManagerCollection::AddOrRemoveTextureStreamingManagerIfNeeded(bool bIsInit)
{
	bool bUseTextureStreaming = false;

#if PLATFORM_SUPPORTS_TEXTURE_STREAMING
	{
		bUseTextureStreaming = CVarSetTextureStreaming.GetValueOnGameThread() != 0;
	}

	if( !GRHISupportsTextureStreaming || IsRunningDedicatedServer() )
	{
		if (bUseTextureStreaming)
		{
			bUseTextureStreaming = false;

			// some code relies on r.TextureStreaming so we're going to disable it here to reflect the hardware capabilities and system needs
			CVarSetTextureStreaming.AsVariable()->Set(0, ECVF_SetByCode);
		}
	}
#endif

	if ( bUseTextureStreaming )
	{
		//Add the texture streaming manager if it's needed.
		if( !RenderAssetStreamingManager )
		{
			FlushRenderingCommands();

			GConfig->GetFloat( TEXT("TextureStreaming"), TEXT("LoadMapTimeLimit"), LoadMapTimeLimit, GEngineIni );
			// Create the streaming manager and add the default streamers.
			RenderAssetStreamingManager = new FRenderAssetStreamingManager();
			AddStreamingManager( RenderAssetStreamingManager );		
				
			// TODO : Register all levels

			//Need to work out if all textures should be streamable and added to the texture streaming manager.
			//This works but may be more heavy handed than necessary.
			if( !bIsInit )
			{
				for( TObjectIterator<UStreamableRenderAsset>It; It; ++It )
				{
					It->LinkStreaming();
				}
			}
		}
	}
	else
	{
		//Remove the texture streaming manager if needed.
		if( RenderAssetStreamingManager )
		{
			FlushRenderingCommands();
			RenderAssetStreamingManager->BlockTillAllRequestsFinished();

			// Stream all LODs back in before disabling the streamer.
			for( TObjectIterator<UStreamableRenderAsset>It; It; ++It )
			{
				if (It->IsStreamable())
				{
					// Force LODs in with high priority, including cinematic ones.
					It->SetForceMipLevelsToBeResident(30.f, 0xFFFFFFFF);
					It->StreamIn(FStreamableRenderResourceState::MAX_LOD_COUNT, true);
				}
			}
			RenderAssetStreamingManager->BlockTillAllRequestsFinished();

			for( TObjectIterator<UStreamableRenderAsset>It; It; ++It )
			{
				It->UnlinkStreaming();
			}

			// Remove unreachable assets from the streamer before it goes away
			UnhashUnreachableObjects(false);

			RemoveStreamingManager(RenderAssetStreamingManager);
			delete RenderAssetStreamingManager;
			RenderAssetStreamingManager = nullptr;
		}
	}
}

/*-----------------------------------------------------------------------------
	Texture streaming helper structs.
-----------------------------------------------------------------------------*/

/**
 * FStreamableTextureInstance serialize operator.
 *
 * @param	Ar					Archive to to serialize object to/ from
 * @param	TextureInstance		Object to serialize
 * @return	Returns the archive passed in
 */
FArchive& operator<<( FArchive& Ar, FStreamableTextureInstance& TextureInstance )
{
	if (Ar.UEVer() >= VER_UE4_STREAMABLE_TEXTURE_AABB)
	{
		Ar << TextureInstance.Bounds;
	}
	else if (Ar.IsLoading())
	{
		FSphere BoundingSphere;
		Ar << BoundingSphere;
		TextureInstance.Bounds = FBoxSphereBounds(BoundingSphere);
	}

	if (Ar.UEVer() >= VER_UE4_STREAMABLE_TEXTURE_MIN_MAX_DISTANCE)
	{
		Ar << TextureInstance.MinDistance;
		Ar << TextureInstance.MaxDistance;
	}
	else if (Ar.IsLoading())
	{
		TextureInstance.MinDistance = 0;
		TextureInstance.MaxDistance = FLT_MAX;
	}

	Ar << TextureInstance.TexelFactor;

	return Ar;
}

/**
 * FDynamicTextureInstance serialize operator.
 *
 * @param	Ar					Archive to to serialize object to/ from
 * @param	TextureInstance		Object to serialize
 * @return	Returns the archive passed in
 */
FArchive& operator<<( FArchive& Ar, FDynamicTextureInstance& TextureInstance )
{
	FStreamableTextureInstance& Super = TextureInstance;
	Ar << Super;

	Ar << TextureInstance.Texture;
	Ar << TextureInstance.bAttached;
	Ar << TextureInstance.OriginalRadius;
	return Ar;
}


/*-----------------------------------------------------------------------------
	Audio chunk handles. Used by the cached audio streaming manager.
-----------------------------------------------------------------------------*/
FAudioChunkHandle::FAudioChunkHandle()
	: CachedData(nullptr)
	, CachedDataNumBytes(0)
	, ChunkIndex(INDEX_NONE)
#if WITH_EDITOR
	, CorrespondingWave(nullptr)
	, ChunkRevision(INDEX_NONE)
#endif
{
} 

FAudioChunkHandle::FAudioChunkHandle(const uint8* InData, uint32 NumBytes, const TSharedRef<const FSoundWaveData>&  InSoundWave, const FName& SoundWaveName, uint32 InChunkIndex, uint64 InCacheLookupID)
	: CachedData(InData)
	, CachedDataNumBytes(NumBytes)
	, CorrespondingWaveName(SoundWaveName)
	, CorrespondingWaveGuid(InSoundWave->GetGUID())
	, ChunkIndex(InChunkIndex)
#if WITH_EDITOR
	, CorrespondingWave(InSoundWave)
	, ChunkRevision(InSoundWave->GetCurrentChunkRevision())
#endif
{
}

FAudioChunkHandle::FAudioChunkHandle(const FAudioChunkHandle& Other)
	: FAudioChunkHandle()
{
	*this = Other;
}

FAudioChunkHandle::FAudioChunkHandle(FAudioChunkHandle&& Other)
	: FAudioChunkHandle()
{
	*this = MoveTemp(Other);
}

FAudioChunkHandle& FAudioChunkHandle::operator=(FAudioChunkHandle&& Other)
{
	// If this chunk was previously referencing another chunk, remove that chunk here.
	if (IsValid())
	{
		IStreamingManager::Get().GetAudioStreamingManager().RemoveReferenceToChunk(*this);
	}

	CachedData = Other.CachedData;
	CachedDataNumBytes = Other.CachedDataNumBytes;
	CorrespondingWaveName = Other.CorrespondingWaveName;
	CorrespondingWaveGuid = Other.CorrespondingWaveGuid;
	ChunkIndex = Other.ChunkIndex;
#if WITH_EDITOR
	CorrespondingWave = MoveTemp(Other.CorrespondingWave);
	ChunkRevision = Other.ChunkRevision;
#endif

	// we don't need to call RemoveReferenceToChunk on Other, nor add a new reference to this chunk, since this is a move.
	// Instead, we can simply null out the other chunk handle without invoking it's destructor.
	Other.CachedData = nullptr;
	Other.CachedDataNumBytes = 0;
	Other.CorrespondingWaveName = FName();
	Other.CorrespondingWaveGuid = FGuid();
	Other.ChunkIndex = INDEX_NONE;
#if WITH_EDITOR
	Other.CorrespondingWave = nullptr;
	Other.ChunkRevision = INDEX_NONE;
#endif

	return *this;
}

FAudioChunkHandle& FAudioChunkHandle::operator=(const FAudioChunkHandle& Other)
{
	// If this chunk was previously referencing another chunk, remove that chunk here.
	if (IsValid())
	{
		IStreamingManager::Get().GetAudioStreamingManager().RemoveReferenceToChunk(*this);
	}

	CachedData = Other.CachedData;
	CachedDataNumBytes = Other.CachedDataNumBytes;
	CorrespondingWaveName = Other.CorrespondingWaveName;
	CorrespondingWaveGuid = Other.CorrespondingWaveGuid;
	ChunkIndex = Other.ChunkIndex;
#if WITH_EDITOR
	CorrespondingWave = Other.CorrespondingWave;
	ChunkRevision = Other.ChunkRevision;
#endif

	if (IsValid())
	{
		// Increment the reference count for the new streaming chunk.
		IStreamingManager::Get().GetAudioStreamingManager().AddReferenceToChunk(*this);
	}

	return *this;
}

FAudioChunkHandle::~FAudioChunkHandle()
{
	if (IsValid())
	{
		IStreamingManager::Get().GetAudioStreamingManager().RemoveReferenceToChunk(*this);
	}
}

const uint8* FAudioChunkHandle::GetData() const
{
	return CachedData;
}

uint32 FAudioChunkHandle::Num() const
{
	return CachedDataNumBytes;
}

bool FAudioChunkHandle::IsValid() const
{
	return (nullptr != GetData());
}

#if WITH_EDITOR
bool FAudioChunkHandle::IsStale() const
{
	TSharedPtr<const FSoundWaveData, ESPMode::ThreadSafe> SoundWaveDataPtr = CorrespondingWave.Pin();

	if (SoundWaveDataPtr.IsValid())
	{
		// NOTE: While this is currently safe in editor, there's no guarantee the USoundWave will be kept alive during the lifecycle of this chunk handle.
		return ChunkRevision != SoundWaveDataPtr->GetCurrentChunkRevision();
	}
	else
	{
		return false;
	}
}
#endif

bool IAudioStreamingManager::RequestChunk(const TSharedRef<const FSoundWaveData>& SoundWave, uint32 ChunkIndex,
	TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted, ENamedThreads::Type ThreadToCallOnLoadCompletedOn, bool bForImmediatePlayback)
{
	static bool bDidWarn = false;
	if (!bDidWarn)
	{
		UE_LOG(LogContentStreaming, Warning, TEXT("This virtual function must be overridden, and will be pure virtual in a future release."));
		bDidWarn = true;
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RequestChunk(FSoundWaveProxy::CreateNonUpdatingCopy(SoundWave), ChunkIndex, OnLoadCompleted, ThreadToCallOnLoadCompletedOn, bForImmediatePlayback);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FAudioChunkHandle IAudioStreamingManager::GetLoadedChunk(const TSharedRef<const FSoundWaveData>& SoundWave, uint32 ChunkIndex, bool bBlockForLoad, bool bForImmediatePlayback) const
{
	static bool bDidWarn = false;
	if (!bDidWarn)
	{
		UE_LOG(LogContentStreaming, Warning, TEXT("This virtual function must be overridden, and will be pure virtual in a future release."));
		bDidWarn = true;
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetLoadedChunk(FSoundWaveProxy::CreateNonUpdatingCopy(SoundWave), ChunkIndex, bBlockForLoad, bForImmediatePlayback);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void IAudioStreamingManager::HandleStarvation(const TSharedPtr<FSoundWaveProxy>& SoundWave, const uint32 ChunkIndex, const uint32 InBytesProduced,
	const uint32 InBytesExpected)
{
	if (ensure(SoundWave.IsValid()))
	{
		HandleStarvation(SoundWave->GetSoundWaveDataRef(), ChunkIndex, InBytesProduced, InBytesExpected);
	}
}

FAudioChunkHandle IAudioStreamingManager::BuildChunkHandle(const uint8* InData, uint32 NumBytes, const TSharedRef<const FSoundWaveData>&  InSoundWaveData, const FName& SoundWaveName, uint32 InChunkIndex, uint64 InCacheLookupID)
{
	return FAudioChunkHandle(InData, NumBytes, InSoundWaveData, SoundWaveName, InChunkIndex, InCacheLookupID);
}

FAudioChunkHandle IAudioStreamingManager::BuildChunkHandle(const uint8* InData, uint32 NumBytes, const FSoundWaveProxyPtr& InSoundWave,
	const FName& SoundWaveName, uint32 InChunkIndex, uint64 CacheLookupID)
{
	if (ensure(InSoundWave.IsValid()))
	{
		return BuildChunkHandle(InData, NumBytes, InSoundWave->GetSoundWaveDataRef(), SoundWaveName, InChunkIndex, CacheLookupID);
	}
		
	return {};
}

void IAudioStreamingManager::LogWarning()
{
	UE_LOGF(LogContentStreaming, Warning, "This function is now deprecated on IAudioStreamingManager and no-ops.");
}

void IStreamingManager::NotifyPrimitiveDetached(const UPrimitiveComponent* Primitive)
{
	NotifyPrimitiveDetached(Primitive->GetPrimitiveComponentInterface());
}

void IStreamingManager::NotifyPrimitiveUpdated(const UPrimitiveComponent* Primitive)
{
	NotifyPrimitiveUpdated(Primitive->GetPrimitiveComponentInterface());
}

void IStreamingManager::NotifyPrimitiveUpdated_Concurrent(const UPrimitiveComponent* Primitive)
{
	NotifyPrimitiveUpdated_Concurrent(Primitive->GetPrimitiveComponentInterface());
}
