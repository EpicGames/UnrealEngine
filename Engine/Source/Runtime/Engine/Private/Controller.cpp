// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Controller.cpp: 

=============================================================================*/
#include "GameFramework/Controller.h"
#include "AI/NavigationSystemBase.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"
#include "NetworkingDistanceConstants.h"
#include "VisualLogger/VisualLogger.h"
#include "GameFramework/Character.h"
#include "GameFramework/GameStateBase.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/PawnMovementComponent.h"
#include "Logging/MessageLog.h"

// @todo this is here only due to circular dependency to AIModule. To be removed
#include "Engine/Canvas.h"

#include "GameFramework/PlayerState.h"
#include "ObjectTrace.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteObject.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(Controller)

DEFINE_LOG_CATEGORY(LogController);
DEFINE_LOG_CATEGORY(LogPath);

#define LOCTEXT_NAMESPACE "Controller"

namespace UE::Gameplay::CVars
{
	bool bAlwaysNotifyClientOnControllerChange = true;
	static FAutoConsoleVariableRef CVarAlwaysNotifyClientOnControllerChange(
		TEXT("Controller.AlwaysNotifyClientOnControllerChange"), bAlwaysNotifyClientOnControllerChange,
		TEXT("If true, OnRep_Controller and NotifyControllerChanged are always called on the client when a pawn's controller changes.\n")
		TEXT("If false, use backward compatible controller notification logic that does not handle networking properly."),
		ECVF_Default);

	float InvalidControlRotationMagnitude = 8388608.f; // 2^23, largest float when fractions are lost, and where FMod loses meaningful precision.
	static FAutoConsoleVariableRef CVarInvalidControlRotationMagnitude(
		TEXT("Controller.InvalidControlRotationMagnitude"), InvalidControlRotationMagnitude,
		TEXT("If any component of an FRotator passed to SetControlRotation is larger than this magnitude, ignore the value. Huge values are usually from uninitialized variables and can cause NaN/Inf to propagate later."),
		ECVF_Default);

	bool bIsControllerPushBased = false;
	static FAutoConsoleVariableRef CVarIsControllerPushBased(
		TEXT("Controller.IsPushBased"), bIsControllerPushBased,
		TEXT("If true, AController's replicated properties will use push-based networking, and will therefore need to be marked dirty when changed."),
		ECVF_Default);
}

AController::AController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	PrimaryActorTick.bCanEverTick = true;
	SetHidden(true);
#if WITH_EDITORONLY_DATA
	bHiddenEd = true;
#endif // WITH_EDITORONLY_DATA
	bOnlyRelevantToOwner = true;

	TransformComponent = CreateDefaultSubobject<USceneComponent>(TEXT("TransformComponent0"));
	RootComponent = TransformComponent;

	SetCanBeDamaged(false);
	bAttachToPawn = false;
	bIsPlayerController = false;
	bCanPossessWithoutAuthority = false;

	if (RootComponent)
	{
		// We attach the RootComponent to the pawn for location updates,
		// but we want to drive rotation with ControlRotation regardless of attachment state.
		RootComponent->SetUsingAbsoluteRotation(true);
	}
}

void AController::K2_DestroyActor()
{
	// do nothing, disallow destroying controller from Blueprints
}

bool AController::IsLocalController() const
{
	const ENetMode NetMode = GetNetMode();

	if (NetMode == NM_Standalone)
	{
		// Not networked.
		return true;
	}
	
	if (NetMode == NM_Client && GetLocalRole() == ROLE_AutonomousProxy)
	{
		// Networked client in control.
		return true;
	}

	if (GetRemoteRole() != ROLE_AutonomousProxy && GetLocalRole() == ROLE_Authority)
	{
		// Local authority in control.
		return true;
	}

	return false;
}

void AController::FailedToSpawnPawn()
{
	ChangeState(NAME_Inactive);
}

void AController::SetInitialLocationAndRotation(const FVector& NewLocation, const FRotator& NewRotation)
{
	SetActorLocationAndRotation(NewLocation, NewRotation, false, nullptr, ETeleportType::ResetPhysics);
	SetControlRotation(NewRotation);
}

FRotator AController::GetControlRotation() const
{
	ControlRotation.DiagnosticCheckNaN();
	return ControlRotation;
}

void AController::SetControlRotation(const FRotator& NewRotation)
{
	if (!IsValidControlRotation(NewRotation))
	{
		logOrEnsureNanError(TEXT("AController::SetControlRotation attempted to apply NaN-containing or NaN-causing rotation! (%s)"), *NewRotation.ToString());
		return;
	}

	if (!ControlRotation.Equals(NewRotation, 1e-3f))
	{
		ControlRotation = NewRotation;

		if (RootComponent && RootComponent->IsUsingAbsoluteRotation())
		{
			RootComponent->SetWorldRotation(GetControlRotation());
		}
	}
	else
	{
		//UE_LOGF(LogPlayerController, Log, "Skipping SetControlRotation %ls for %ls (Pawn %ls)", *NewRotation.ToString(), *GetNameSafe(this), *GetNameSafe(GetPawn()));
	}
}

bool AController::IsValidControlRotation(FRotator CheckRotation) const
{
	if (CheckRotation.ContainsNaN())
	{
		return false;
	}

	// Really large values can be technically valid but are usually the result of uninitialized values, and those can cause
	// conversion to FQuat or Vector to fail and generate NaN or Inf.
	if (FMath::Abs(CheckRotation.Pitch) >= UE::Gameplay::CVars::InvalidControlRotationMagnitude ||
		FMath::Abs(CheckRotation.Yaw  ) >= UE::Gameplay::CVars::InvalidControlRotationMagnitude ||
		FMath::Abs(CheckRotation.Roll ) >= UE::Gameplay::CVars::InvalidControlRotationMagnitude)
	{
		return false;
	}

	return true;
}


void AController::SetIgnoreMoveInput(bool bNewMoveInput)
{
	IgnoreMoveInput = FMath::Max(IgnoreMoveInput + (bNewMoveInput ? +1 : -1), 0);
}

void AController::ResetIgnoreMoveInput()
{
	IgnoreMoveInput = 0;
}

bool AController::IsMoveInputIgnored() const
{
	return (IgnoreMoveInput > 0);
}

void AController::SetIgnoreLookInput(bool bNewLookInput)
{
	IgnoreLookInput = FMath::Max(IgnoreLookInput + (bNewLookInput ? +1 : -1), 0);
}

void AController::ResetIgnoreLookInput()
{
	IgnoreLookInput = 0;
}

bool AController::IsLookInputIgnored() const
{
	return (IgnoreLookInput > 0);
}

void AController::ResetIgnoreInputFlags()
{
	ResetIgnoreMoveInput();
	ResetIgnoreLookInput();
}


void AController::AttachToPawn(APawn* InPawn)
{
	if (bAttachToPawn && RootComponent)
	{
		if (InPawn)
		{
			// Only attach if not already attached.
			if (InPawn->GetRootComponent() && RootComponent->GetAttachParent() != InPawn->GetRootComponent())
			{
				RootComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
				RootComponent->SetRelativeLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator);
				RootComponent->AttachToComponent(InPawn->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
			}
		}
		else
		{
			DetachFromPawn();
		}
	}
}

void AController::DetachFromPawn()
{
	if (bAttachToPawn && RootComponent && RootComponent->GetAttachParent() && Cast<APawn>(RootComponent->GetAttachmentRootActor()))
	{
		RootComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
	}
}


AActor* AController::GetViewTarget() const
{
	if (Pawn)
	{
		return Pawn;
	}

	return const_cast<AController*>(this);
}

void AController::GetPlayerViewPoint( FVector& out_Location, FRotator& out_Rotation ) const
{
	GetActorEyesViewPoint( out_Location, out_Rotation);
}

bool AController::LineOfSightTo(const AActor* Other, FVector ViewPoint, bool bAlternateChecks) const
{
	if( !Other )
	{
		return false;
	}

	if ( ViewPoint.IsZero() )
	{
		FRotator ViewRotation;
		GetActorEyesViewPoint(ViewPoint, ViewRotation);
	}

	FCollisionQueryParams CollisionParms(SCENE_QUERY_STAT(LineOfSight), true, Other);
	CollisionParms.AddIgnoredActor(this->GetPawn());
	FVector TargetLocation = Other->GetTargetLocation(Pawn);
	bool bHit = GetWorld()->LineTraceTestByChannel(ViewPoint, TargetLocation, ECC_Visibility, CollisionParms);
	if( !bHit )
	{
		return true;
	}

	// if other isn't using a cylinder for collision and isn't a Pawn (which already requires an accurate cylinder for AI)
	// then don't go any further as it likely will not be tracing to the correct location
	if (!Cast<const APawn>(Other) && Cast<UCapsuleComponent>(Other->GetRootComponent()) == NULL)
	{
		return false;
	}
	float distSq = (Other->GetActorLocation() - ViewPoint).SizeSquared();
	if ( distSq > FARSIGHTTHRESHOLDSQUARED )
	{
		return false;
	}
	if ( !Cast<const APawn>(Other) && (distSq > NEARSIGHTTHRESHOLDSQUARED) ) 
	{
		return false;
	}

	float OtherRadius, OtherHeight;
	Other->GetSimpleCollisionCylinder(OtherRadius, OtherHeight);
	
	//try viewpoint to head
	bHit = GetWorld()->LineTraceTestByChannel(ViewPoint,  Other->GetActorLocation() + FVector(0.f,0.f,OtherHeight), ECC_Visibility, CollisionParms);
	return !bHit;
}

void AController::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	if (IsValidChecked(this))
	{
		GetWorld()->AddController( this );

		// Since we avoid updating rotation in SetControlRotation() if it hasn't changed,
		// we should make sure that the initial RootComponent rotation matches it if ControlRotation was set directly.
		if (RootComponent && RootComponent->IsUsingAbsoluteRotation())
		{
			RootComponent->SetWorldRotation(GetControlRotation());
		}
	}
}

void AController::Possess(APawn* InPawn)
{
	if (!bCanPossessWithoutAuthority && !HasAuthority())
	{
		FMessageLog("PIE").Warning(FText::Format(
			LOCTEXT("ControllerPossessAuthorityOnly", "Possess function should only be used by the network authority for {0}"),
			FText::FromName(GetFName())
			));
		UE_LOGF(LogController, Warning, "Trying to possess %ls without network authority! Request will be ignored.", *GetNameSafe(InPawn));
		return;
	}

	REDIRECT_OBJECT_TO_VLOG(InPawn, this);

	APawn* CurrentPawn = GetPawn();

	// A notification is required when the current assigned pawn is not possessed (i.e. pawn assigned before calling Possess)
	const bool bNotificationRequired = (CurrentPawn != nullptr) && (CurrentPawn->GetController() == nullptr);

	// To preserve backward compatibility we keep notifying derived classed for null pawn in case some
	// overrides decided to react differently when asked to possess a null pawn.
	// Default engine implementation is to unpossess the current pawn.
	OnPossess(InPawn);

	// Notify when pawn to possess (different than the assigned one) has been accepted by the native class or notification is explicitly required
	APawn* NewPawn = GetPawn();
	if ((NewPawn != CurrentPawn) || bNotificationRequired)
	{
		ReceivePossess(NewPawn);
		OnNewPawn.Broadcast(NewPawn);
		OnPossessedPawnChanged.Broadcast(bNotificationRequired ? nullptr : CurrentPawn, NewPawn);
	}
	
	TRACE_PAWN_POSSESS(this, InPawn); 
}

void AController::OnPossess(APawn* InPawn)
{
	const bool bNewPawn = GetPawn() != InPawn;

	// Unpossess current pawn (if any) when current pawn changes
	if (bNewPawn && GetPawn() != nullptr)
	{
		UnPossess();
	}

	if (InPawn == nullptr)
	{
		return;
	}

	if (InPawn->GetController() != nullptr)
	{
		UE_CLOGF(InPawn->GetController() == this, LogController, Warning, "Asking %ls to possess pawn %ls more than once; pawn will be restarted! Should call Unpossess first.", *GetNameSafe(this), *GetNameSafe(InPawn));
		InPawn->GetController()->UnPossess();
	}

	InPawn->PossessedBy(this);
	SetPawn(InPawn);

	// update rotation to match possessed pawn's rotation
	SetControlRotation(Pawn->GetActorRotation());

	Pawn->DispatchRestart(false);
}

void AController::UnPossess()
{
	APawn* CurrentPawn = GetPawn();

	// No need to notify if we don't have a pawn
	if (CurrentPawn == nullptr)
	{
		return;
	}

	OnUnPossess();

	// Notify only when pawn has been successfully unpossessed by the native class.
	APawn* NewPawn = GetPawn();
	if (NewPawn != CurrentPawn)
	{
		ReceiveUnPossess(CurrentPawn);
		OnNewPawn.Broadcast(NewPawn);
		OnPossessedPawnChanged.Broadcast(CurrentPawn, NewPawn);
	}
	
	TRACE_PAWN_POSSESS(this, (APawn*)nullptr) 
}

void AController::OnUnPossess()
{
	// Should not be called when Pawn is null but since OnUnPossess could be overridden
	// the derived class could have already cleared the pawn and then call its base class.
	if ( Pawn != NULL )
	{
		Pawn->UnPossessed();
		SetPawn(NULL);
	}
}

void AController::PawnPendingDestroy(APawn* inPawn)
{
	if ( IsInState(NAME_Inactive) )
	{
		UE_LOGF(LogController, Log, "PawnPendingDestroy while inactive %ls", *GetName());
	}

	if ( inPawn != Pawn )
	{
		return;
	}

	UnPossess();
	ChangeState(NAME_Inactive);

	if (PlayerState == NULL)
	{
		Destroy();
	}
}

void AController::Reset()
{
	Super::Reset();
	StartSpot = NULL;
}

/// @cond DOXYGEN_WARNINGS

bool AController::ClientSetLocation_Validate(FVector NewLocation, FRotator NewRotation)
{
	return true;
}

void AController::ClientSetLocation_Implementation( FVector NewLocation, FRotator NewRotation )
{
	ClientSetRotation(NewRotation);
	if (Pawn != NULL)
	{
		Pawn->TeleportTo(NewLocation, Pawn->GetActorRotation());
	}
}

bool AController::ClientSetRotation_Validate(FRotator NewRotation, bool bResetCamera)
{
	return true;
}

void AController::ClientSetRotation_Implementation( FRotator NewRotation, bool bResetCamera )
{
	SetControlRotation(NewRotation);
	if (Pawn != NULL)
	{
		Pawn->FaceRotation( NewRotation, 0.f );
	}
}

/// @endcond

void AController::RemovePawnTickDependency(APawn* InOldPawn)
{
	if (InOldPawn != NULL)
	{
		UPawnMovementComponent* PawnMovement = InOldPawn->GetMovementComponent();
		if (PawnMovement)
		{
			PawnMovement->PrimaryComponentTick.RemovePrerequisite(this, this->PrimaryActorTick);
		}
		
		InOldPawn->PrimaryActorTick.RemovePrerequisite(this, this->PrimaryActorTick);
	}
}


void AController::AddPawnTickDependency(APawn* NewPawn)
{
	if (NewPawn != NULL)
	{
		bool bNeedsPawnPrereq = true;
		UPawnMovementComponent* PawnMovement = NewPawn->GetMovementComponent();
		if (PawnMovement && PawnMovement->PrimaryComponentTick.bCanEverTick)
		{
			PawnMovement->PrimaryComponentTick.AddPrerequisite(this, this->PrimaryActorTick);

			// Don't need a prereq on the pawn if the movement component already sets up a prereq.
			if (PawnMovement->bTickBeforeOwner || NewPawn->PrimaryActorTick.GetPrerequisites().Contains(FTickPrerequisite(PawnMovement, PawnMovement->PrimaryComponentTick)))
			{
				bNeedsPawnPrereq = false;
			}
		}
		
		if (bNeedsPawnPrereq)
		{
			NewPawn->PrimaryActorTick.AddPrerequisite(this, this->PrimaryActorTick);
		}
	}
}

void AController::SetPawn_Direct(APawn* InPawn)
{
	if (UE::Gameplay::CVars::bIsControllerPushBased)
	{
		COMPARE_ASSIGN_AND_MARK_PROPERTY_DIRTY(AController, Pawn, InPawn, this);
		return;
	}

	Pawn = InPawn;
}

void AController::SetPawn(APawn* InPawn)
{
	RemovePawnTickDependency(Pawn);

	SetPawn_Direct(InPawn);
	Character = (Pawn ? Cast<ACharacter>(Pawn) : NULL);

	AttachToPawn(Pawn);

	AddPawnTickDependency(Pawn);
}

void AController::SetPawnFromRep(APawn* InPawn)
{
	// This function is needed to ensure OnRep_Pawn is called in the case we need to set AController::Pawn
	// due to APawn::Controller being replicated first. See additional notes in APawn::OnRep_Controller.
	RemovePawnTickDependency(Pawn);
	SetPawn_Direct(InPawn);
	OnRep_Pawn();
}

void AController::OnRep_Pawn()
{
	APawn* StrongOldPawn = OldPawn.Get();
	// Detect when pawn changes, so we can NULL out the controller on the old pawn
	if ((StrongOldPawn != nullptr) && (Pawn != StrongOldPawn) && (StrongOldPawn->GetController() == this))
	{
		// Set the old controller to NULL, since we are no longer the owner, and can't rely on it replicating to us anymore
		StrongOldPawn->SetController(nullptr);
		if (UE::Gameplay::CVars::bAlwaysNotifyClientOnControllerChange)
		{
			// This will notify other systems like the game instance
			StrongOldPawn->OnRep_Controller();
		}
	}

	OldPawn = Pawn;

	SetPawn(Pawn);

	if (StrongOldPawn != Pawn)
	{
		OnPossessedPawnChanged.Broadcast(StrongOldPawn, Pawn);
	}
}

void AController::SetPlayerState(APlayerState* InPlayerState)
{
	if (UE::Gameplay::CVars::bIsControllerPushBased)
	{
		COMPARE_ASSIGN_AND_MARK_PROPERTY_DIRTY(AController, PlayerState, InPlayerState, this);
		return;
	}

	PlayerState = InPlayerState;
}

void AController::OnRep_PlayerState()
{
	if (PlayerState != NULL)
	{
		PlayerState->ClientInitialize(this);
	}
}

void AController::Destroyed()
{
	if (GetLocalRole() == ROLE_Authority && PlayerState != NULL)
	{
		// if we are a player, log out
		AGameModeBase* const GameMode = GetWorld()->GetAuthGameMode();
		if (GameMode)
		{
			GameMode->Logout(this);
		}

		CleanupPlayerState();
	}

	UnPossess();
	GetWorld()->RemoveController( this );
	Super::Destroyed();
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
FRemoteServerId AController::GetPreferredRemoteServer() const
{
	TObjectPtr<APawn> PawnPtr = GetPawn();  // TObjectPtr by value, no auto-pull
	if (!AutoRTFM::IsClosed() && PawnPtr.GetResidence() == EResidence::Remote)
	{
		// We are outside of a transaction (probably push migration step) and the
		// Pawn is already on another server — that IS its preferred server
		FRemoteServerId PawnServerId = UE::RemoteObject::Handle::GetOwnerServerId(PawnPtr.GetRemoteId());
		if (PawnServerId.IsValid())
		{
			return PawnServerId;
		}
	}
	else if (APawn* LocalPawn = PawnPtr.Get())
	{
		// Pawn is local — ask where it WANTS to be
		return LocalPawn->GetPreferredRemoteServer();
	}

	// No Pawn — stay where we are (don't use spatial delegate; controllers
	// without pawns have no meaningful location to determine preferred server)
	return UE::RemoteObject::Handle::GetOwnerServerId(this);
}
#endif

void AController::CleanupPlayerState()
{
	PlayerState->Destroy();
	SetPlayerState(NULL);
}

void AController::InstigatedAnyDamage(float Damage, const class UDamageType* DamageType, class AActor* DamagedActor, class AActor* DamageCauser)
{
	ReceiveInstigatedAnyDamage(Damage, DamageType, DamagedActor, DamageCauser);
	OnInstigatedAnyDamage.Broadcast(Damage, DamageType, DamagedActor, DamageCauser);
}

void AController::InitPlayerState()
{
	if ( GetNetMode() != NM_Client )
	{
		UWorld* const World = GetWorld();
		const AGameModeBase* GameMode = World ? World->GetAuthGameMode() : NULL;

		// If the GameMode is null, this might be a network client that's trying to
		// record a replay. Try to use the default game mode in this case so that
		// we can still spawn a PlayerState.
		if (GameMode == NULL)
		{
			const AGameStateBase* const GameState = World ? World->GetGameState() : NULL;
			GameMode = GameState ? GameState->GetDefaultGameMode() : NULL;
		}

		if (GameMode != NULL)
		{
			FActorSpawnParameters SpawnInfo;
			SpawnInfo.Owner = this;
			SpawnInfo.Instigator = GetInstigator();
			SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			SpawnInfo.ObjectFlags |= RF_Transient;	// We never want player states to save into a map

			TSubclassOf<APlayerState> PlayerStateClassToSpawn = GameMode->PlayerStateClass;
			if (PlayerStateClassToSpawn.Get() == nullptr)
			{
				UE_LOGF(LogPlayerController, Log, "AController::InitPlayerState: the PlayerStateClass of game mode %ls is null, falling back to APlayerState.", *GameMode->GetName());
				PlayerStateClassToSpawn = APlayerState::StaticClass();
			}

			SetPlayerState(World->SpawnActor<APlayerState>(PlayerStateClassToSpawn, SpawnInfo));
	
			// force a default player name if necessary
			if (PlayerState && PlayerState->GetPlayerName().IsEmpty())
			{
				// don't call SetPlayerName() as that will broadcast entry messages but the GameMode hasn't had a chance
				// to potentially apply a player/bot name yet
				
				PlayerState->SetPlayerNameInternal(GameMode->DefaultPlayerName.ToString());
			}
		}
	}
}


void AController::GameHasEnded(AActor* EndGameFocus, bool bIsWinner)
{
}


FRotator AController::GetDesiredRotation() const
{
	return GetControlRotation();
}


void AController::GetActorEyesViewPoint( FVector& out_Location, FRotator& out_Rotation ) const
{
	// If we have a Pawn, this is our view point.
	if ( Pawn != NULL )
	{
		Pawn->GetActorEyesViewPoint( out_Location, out_Rotation );
	}
	// otherwise, controllers don't have a physical location
}


void AController::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	if ( Pawn == NULL )
	{
		if (PlayerState == NULL)
		{
			DisplayDebugManager.DrawString(TEXT("NO PlayerState"));
		}
		else
		{
			PlayerState->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
		}
		Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);
		return;
	}

	DisplayDebugManager.SetDrawColor(FColor(255, 0, 0));
	DisplayDebugManager.DrawString(FString::Printf(TEXT("CONTROLLER %s Pawn %s"), *GetName(), *Pawn->GetName()));
}

FString AController::GetHumanReadableName() const
{
	return PlayerState ? PlayerState->GetPlayerName() : *GetName();
}

void AController::CurrentLevelUnloaded() {}

void AController::ChangeState(FName NewState)
{
	if(NewState != StateName)
	{
		// end current state
		if(StateName == NAME_Inactive)
		{
			EndInactiveState();
		}

		// Set new state name
		StateName = NewState;

		// start new state
		if(StateName == NAME_Inactive)
		{
			BeginInactiveState();
		}
	}
}

FName AController::GetStateName() const
{
	return StateName;
}

bool AController::IsInState(FName InStateName) const
{
	return (StateName == InStateName);
}

void AController::BeginInactiveState() {}

void AController::EndInactiveState() {}

APawn* AController::K2_GetPawn() const
{
	return GetPawn();
}

const FNavAgentProperties& AController::GetNavAgentPropertiesRef() const
{
	return Pawn ? Pawn->GetNavAgentPropertiesRef() : FNavAgentProperties::DefaultProperties;
}

FVector AController::GetNavAgentLocation() const
{
	return Pawn ? Pawn->GetNavAgentLocation() : FVector::ZeroVector;
}

void AController::GetMoveGoalReachTest(const AActor* MovingActor, const FVector& MoveOffset, FVector& GoalOffset, float& GoalRadius, float& GoalHalfHeight) const 
{
	if (Pawn)
	{
		Pawn->GetMoveGoalReachTest(MovingActor, MoveOffset, GoalOffset, GoalRadius, GoalHalfHeight); 
	}
}

bool AController::ShouldPostponePathUpdates() const
{
	return Pawn ? Pawn->ShouldPostponePathUpdates() : false;
}

bool AController::IsFollowingAPath() const
{
	return FNavigationSystem::IsFollowingAPath(*this);
}

IPathFollowingAgentInterface* AController::GetPathFollowingAgent() const
{
	return FNavigationSystem::FindPathFollowingAgentForActor(*this);
}

void AController::StopMovement()
{
	FNavigationSystem::StopMovement(*this);
}

void AController::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = UE::Gameplay::CVars::bIsControllerPushBased;
	DOREPLIFETIME_WITH_PARAMS_FAST(AController, PlayerState, Params);
	Params.RepNotifyCondition = REPNOTIFY_Always;
	DOREPLIFETIME_WITH_PARAMS_FAST(AController, Pawn, Params);
}

bool AController::ShouldParticipateInSeamlessTravel() const
{
	return (PlayerState != nullptr);
}

#undef LOCTEXT_NAMESPACE

