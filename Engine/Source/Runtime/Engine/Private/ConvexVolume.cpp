// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ConvexVolume.cpp: Convex volume implementation.
=============================================================================*/

#include "ConvexVolume.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/Polys.h"

/**
 * Builds the permuted planes for SIMD fast clipping
 */
void FConvexVolume::Init(void)
{
	int32 NumToAdd = Planes.Num() / 4;
	int32 NumRemaining = Planes.Num() % 4;
	// Presize the array
	PermutedPlanes.Empty(NumToAdd * 4 + (NumRemaining ? 4 : 0));
	// For each set of four planes
	for (int32 Count = 0, Offset = 0; Count < NumToAdd; Count++, Offset += 4)
	{
		// Add them in SSE ready form
		new(PermutedPlanes)FPlane(Planes[Offset + 0].X,Planes[Offset + 1].X,Planes[Offset + 2].X,Planes[Offset + 3].X);
		new(PermutedPlanes)FPlane(Planes[Offset + 0].Y,Planes[Offset + 1].Y,Planes[Offset + 2].Y,Planes[Offset + 3].Y);
		new(PermutedPlanes)FPlane(Planes[Offset + 0].Z,Planes[Offset + 1].Z,Planes[Offset + 2].Z,Planes[Offset + 3].Z);
		new(PermutedPlanes)FPlane(Planes[Offset + 0].W,Planes[Offset + 1].W,Planes[Offset + 2].W,Planes[Offset + 3].W);
	}
	// Pad the last set so we have an even 4 planes of vert data
	if (NumRemaining)
	{
		FPlane Last1, Last2, Last3, Last4;
		// Read the last set of verts
		switch (NumRemaining)
		{
			case 3:
			{
				Last1 = Planes[NumToAdd * 4 + 0];
				Last2 = Planes[NumToAdd * 4 + 1];
				Last3 = Planes[NumToAdd * 4 + 2];
				Last4 = Last1;
				break;
			}
			case 2:
			{
				Last1 = Planes[NumToAdd * 4 + 0];
				Last2 = Planes[NumToAdd * 4 + 1];
				Last3 = Last4 = Last1;
				break;
			}
			case 1:
			{
				Last1 = Planes[NumToAdd * 4 + 0];
				Last2 = Last3 = Last4 = Last1;
				break;
			}
			default:
			{
				Last1 = FPlane(0, 0, 0, 0);
				Last2 = Last3 = Last4 = Last1;
				break;
			}
		}
		// Add them in SIMD ready form
		new(PermutedPlanes)FPlane(Last1.X,Last2.X,Last3.X,Last4.X);
		new(PermutedPlanes)FPlane(Last1.Y,Last2.Y,Last3.Y,Last4.Y);
		new(PermutedPlanes)FPlane(Last1.Z,Last2.Z,Last3.Z,Last4.Z);
		new(PermutedPlanes)FPlane(Last1.W,Last2.W,Last3.W,Last4.W);
	}
}

//
//	FConvexVolume::ClipPolygon
//

bool FConvexVolume::ClipPolygon(FPoly& Polygon) const
{
	for(int32 PlaneIndex = 0;PlaneIndex < Planes.Num();PlaneIndex++)
	{
		const FPlane&	Plane = Planes[PlaneIndex];
		if(!Polygon.Split(-FVector3f(Plane), (FVector3f)Plane * Plane.W))
			return 0;
	}
	return 1;
}


//
//	FConvexVolume::GetBoxIntersectionOutcode
//

FOutcode FConvexVolume::GetBoxIntersectionOutcode(const FVector& Origin,const FVector& Extent) const
{
	FOutcode Result(true,false);

	checkSlow(PermutedPlanes.Num() % 4 == 0);

	// Load the origin & extent
	VectorRegister Orig = VectorLoadFloat3(&Origin);
	VectorRegister Ext = VectorLoadFloat3(&Extent);
	// Splat origin into 3 vectors
	VectorRegister OrigX = VectorReplicate(Orig, 0);
	VectorRegister OrigY = VectorReplicate(Orig, 1);
	VectorRegister OrigZ = VectorReplicate(Orig, 2);
	// Splat extent into 3 vectors
	VectorRegister ExtentX = VectorReplicate(Ext, 0);
	VectorRegister ExtentY = VectorReplicate(Ext, 1);
	VectorRegister ExtentZ = VectorReplicate(Ext, 2);
	// Splat the abs for the pushout calculation
	VectorRegister AbsExt = VectorAbs(Ext);
	VectorRegister AbsExtentX = VectorReplicate(AbsExt, 0);
	VectorRegister AbsExtentY = VectorReplicate(AbsExt, 1);
	VectorRegister AbsExtentZ = VectorReplicate(AbsExt, 2);
	// Since we are moving straight through get a pointer to the data
	const FPlane* RESTRICT PermutedPlanePtr = (FPlane*)PermutedPlanes.GetData();
	// Process four planes at a time until we have < 4 left
	for (int32 Count = 0; Count < PermutedPlanes.Num(); Count += 4)
	{
		// Load 4 planes that are already all Xs, Ys, ...
		VectorRegister PlanesX = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesY = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesZ = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesW = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		// Calculate the distance (x * x) + (y * y) + (z * z) - w
		VectorRegister DistX = VectorMultiply(OrigX,PlanesX);
		VectorRegister DistY = VectorMultiplyAdd(OrigY,PlanesY,DistX);
		VectorRegister DistZ = VectorMultiplyAdd(OrigZ,PlanesZ,DistY);
		VectorRegister Distance = VectorSubtract(DistZ,PlanesW);
		// Now do the push out FMath::Abs(x * x) + FMath::Abs(y * y) + FMath::Abs(z * z)
		VectorRegister PushX = VectorMultiply(AbsExtentX,VectorAbs(PlanesX));
		VectorRegister PushY = VectorMultiplyAdd(AbsExtentY,VectorAbs(PlanesY),PushX);
		VectorRegister PushOut = VectorMultiplyAdd(AbsExtentZ,VectorAbs(PlanesZ),PushY);

		// Check for completely outside
		if (VectorAnyGreaterThan(Distance,PushOut))
		{
			Result.SetInside(false);
			Result.SetOutside(true);
			break;
		}

		// See if any part is outside
		if (VectorAnyGreaterThan(Distance,VectorNegate(PushOut)))
		{
			Result.SetOutside(true);
		}
	}

	return Result;
}

//
//	FConvexVolume::IntersectBoxWithPermutedPlanes
//

static FORCEINLINE bool IntersectBoxWithPermutedPlanes(
	const FConvexVolume::FPermutedPlaneArray& PermutedPlanes,
	const VectorRegister& BoxOrigin,
	const VectorRegister& BoxExtent )
{
	checkSlow(PermutedPlanes.Num() % 4 == 0);

	// Splat origin into 3 vectors
	VectorRegister OrigX = VectorReplicate(BoxOrigin, 0);
	VectorRegister OrigY = VectorReplicate(BoxOrigin, 1);
	VectorRegister OrigZ = VectorReplicate(BoxOrigin, 2);
	// Splat extent into 3 vectors
	VectorRegister ExtentX = VectorReplicate(BoxExtent, 0);
	VectorRegister ExtentY = VectorReplicate(BoxExtent, 1);
	VectorRegister ExtentZ = VectorReplicate(BoxExtent, 2);
	// Splat the abs for the pushout calculation
	VectorRegister AbsExt = VectorAbs(BoxExtent);
	VectorRegister AbsExtentX = VectorReplicate(AbsExt, 0);
	VectorRegister AbsExtentY = VectorReplicate(AbsExt, 1);
	VectorRegister AbsExtentZ = VectorReplicate(AbsExt, 2);
	// Since we are moving straight through get a pointer to the data
	const FPlane* RESTRICT PermutedPlanePtr = (FPlane*)PermutedPlanes.GetData();
	// Process four planes at a time until we have < 4 left
	for ( int32 Count = 0, Num = PermutedPlanes.Num(); Count < Num; Count += 4 )
	{
		// Load 4 planes that are already all Xs, Ys, ...
		VectorRegister PlanesX = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesY = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesZ = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesW = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		// Calculate the distance (x * x) + (y * y) + (z * z) - w
		VectorRegister DistX = VectorMultiply(OrigX,PlanesX);
		VectorRegister DistY = VectorMultiplyAdd(OrigY,PlanesY,DistX);
		VectorRegister DistZ = VectorMultiplyAdd(OrigZ,PlanesZ,DistY);
		VectorRegister Distance = VectorSubtract(DistZ,PlanesW);
		// Now do the push out FMath::Abs(x * x) + FMath::Abs(y * y) + FMath::Abs(z * z)
		VectorRegister PushX = VectorMultiply(AbsExtentX,VectorAbs(PlanesX));
		VectorRegister PushY = VectorMultiplyAdd(AbsExtentY,VectorAbs(PlanesY),PushX);
		VectorRegister PushOut = VectorMultiplyAdd(AbsExtentZ,VectorAbs(PlanesZ),PushY);

		// Check for completely outside
		if ( VectorAnyGreaterThan(Distance,PushOut) )
		{
			return false;
		}
	}
	return true;
}

//
//	FConvexVolume::IntersectBox
//

bool FConvexVolume::IntersectBox(const FVector& Origin,const FVector& Extent) const
{
	// Load the origin & extent
	const VectorRegister Orig = VectorLoadFloat3( &Origin );
	const VectorRegister Ext = VectorLoadFloat3( &Extent );
	return IntersectBoxWithPermutedPlanes( PermutedPlanes, Orig, Ext );
}

//
//	FConvexVolume::IntersectBox
//

bool FConvexVolume::IntersectBox( const FVector& Origin,const FVector& Translation,const FVector& Extent ) const
{
	const VectorRegister Orig = VectorLoadFloat3( &Origin );
	const VectorRegister Trans = VectorLoadFloat3( &Translation );	
	const VectorRegister BoxExtent = VectorLoadFloat3( &Extent );
	const VectorRegister BoxOrigin = VectorAdd( Orig, Trans );
	return IntersectBoxWithPermutedPlanes( PermutedPlanes, BoxOrigin, BoxExtent );
}

//
//	FConvexVolume::IntersectSphere with the addition check of if the sphere is COMPLETELY contained or only partially contained
//

bool FConvexVolume::IntersectBox(const FVector& Origin,const FVector& Extent, bool& bOutFullyContained) const
{
	// Assume fully contained
	bOutFullyContained = true;

	checkSlow(PermutedPlanes.Num() % 4 == 0);

	// Load the origin & extent
	VectorRegister Orig = VectorLoadFloat3(&Origin);
	VectorRegister Ext = VectorLoadFloat3(&Extent);
	// Splat origin into 3 vectors
	VectorRegister OrigX = VectorReplicate(Orig, 0);
	VectorRegister OrigY = VectorReplicate(Orig, 1);
	VectorRegister OrigZ = VectorReplicate(Orig, 2);
	// Splat extent into 3 vectors
	VectorRegister ExtentX = VectorReplicate(Ext, 0);
	VectorRegister ExtentY = VectorReplicate(Ext, 1);
	VectorRegister ExtentZ = VectorReplicate(Ext, 2);
	// Splat the abs for the pushout calculation
	VectorRegister AbsExt = VectorAbs(Ext);
	VectorRegister AbsExtentX = VectorReplicate(AbsExt, 0);
	VectorRegister AbsExtentY = VectorReplicate(AbsExt, 1);
	VectorRegister AbsExtentZ = VectorReplicate(AbsExt, 2);
	// Since we are moving straight through get a pointer to the data
	const FPlane* RESTRICT PermutedPlanePtr = (FPlane*)PermutedPlanes.GetData();
	// Process four planes at a time until we have < 4 left
	for (int32 Count = 0; Count < PermutedPlanes.Num(); Count += 4)
	{
		// Load 4 planes that are already all Xs, Ys, ...
		VectorRegister PlanesX = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesY = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesZ = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesW = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		// Calculate the distance (x * x) + (y * y) + (z * z) - w
		VectorRegister DistX = VectorMultiply(OrigX,PlanesX);
		VectorRegister DistY = VectorMultiplyAdd(OrigY,PlanesY,DistX);
		VectorRegister DistZ = VectorMultiplyAdd(OrigZ,PlanesZ,DistY);
		VectorRegister Distance = VectorSubtract(DistZ,PlanesW);
		// Now do the push out FMath::Abs(x * x) + FMath::Abs(y * y) + FMath::Abs(z * z)
		VectorRegister PushX = VectorMultiply(AbsExtentX,VectorAbs(PlanesX));
		VectorRegister PushY = VectorMultiplyAdd(AbsExtentY,VectorAbs(PlanesY),PushX);
		VectorRegister PushOut = VectorMultiplyAdd(AbsExtentZ,VectorAbs(PlanesZ),PushY);
		VectorRegister PushOutNegative = VectorNegate(PushOut);

		// Check for completely outside
		if (VectorAnyGreaterThan(Distance,PushOut))
		{
			bOutFullyContained = false;
			return false;
		}

		// Definitely inside frustums, but check to see if it's fully contained
		if (VectorAnyGreaterThan(Distance,PushOutNegative))
		{
			bOutFullyContained = false;
		}
	}
	return true;
}

//
//	FConvexVolume::IntersectSphere
//

bool FConvexVolume::IntersectSphere(const FVector& Origin,const float& Radius) const
{
	checkSlow(PermutedPlanes.Num() % 4 == 0);

	// Load the origin & radius
	VectorRegister Orig = VectorLoadFloat3(&Origin);
	VectorRegister VRadius = VectorLoadFloat1(&Radius);
	// Splat origin into 3 vectors
	VectorRegister OrigX = VectorReplicate(Orig, 0);
	VectorRegister OrigY = VectorReplicate(Orig, 1);
	VectorRegister OrigZ = VectorReplicate(Orig, 2);
	// Since we are moving straight through get a pointer to the data
	const FPlane* RESTRICT PermutedPlanePtr = (FPlane*)PermutedPlanes.GetData();
	// Process four planes at a time until we have < 4 left
	for (int32 Count = 0; Count < PermutedPlanes.Num(); Count += 4)
	{
		// Load 4 planes that are already all Xs, Ys, ...
		VectorRegister PlanesX = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesY = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesZ = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesW = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		// Calculate the distance (x * x) + (y * y) + (z * z) - w
		VectorRegister DistX = VectorMultiply(OrigX,PlanesX);
		VectorRegister DistY = VectorMultiplyAdd(OrigY,PlanesY,DistX);
		VectorRegister DistZ = VectorMultiplyAdd(OrigZ,PlanesZ,DistY);
		VectorRegister Distance = VectorSubtract(DistZ,PlanesW);

		// Check for completely outside
		if (VectorAnyGreaterThan(Distance,VRadius))
		{
			return false;
		}
	}
	return true;
}


//
//	FConvexVolume::IntersectSphere with the addition check of if the sphere is COMPLETELY contained or only partially contained
//

bool FConvexVolume::IntersectSphere(const FVector& Origin,const float& Radius, bool& bOutFullyContained) const
{
	//Assume fully contained
	bOutFullyContained = true;

	checkSlow(PermutedPlanes.Num() % 4 == 0);

	// Load the origin & radius
	VectorRegister Orig = VectorLoadFloat3(&Origin);
	VectorRegister VRadius = VectorLoadFloat1(&Radius);
	VectorRegister NegativeVRadius = VectorNegate(VRadius);
	// Splat origin into 3 vectors
	VectorRegister OrigX = VectorReplicate(Orig, 0);
	VectorRegister OrigY = VectorReplicate(Orig, 1);
	VectorRegister OrigZ = VectorReplicate(Orig, 2);
	// Since we are moving straight through get a pointer to the data
	const FPlane* RESTRICT PermutedPlanePtr = (FPlane*)PermutedPlanes.GetData();
	// Process four planes at a time until we have < 4 left
	for (int32 Count = 0; Count < PermutedPlanes.Num(); Count += 4)
	{
		// Load 4 planes that are already all Xs, Ys, ...
		VectorRegister PlanesX = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesY = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesZ = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesW = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		// Calculate the distance (x * x) + (y * y) + (z * z) - w
		VectorRegister DistX = VectorMultiply(OrigX,PlanesX);
		VectorRegister DistY = VectorMultiplyAdd(OrigY,PlanesY,DistX);
		VectorRegister DistZ = VectorMultiplyAdd(OrigZ,PlanesZ,DistY);
		VectorRegister Distance = VectorSubtract(DistZ,PlanesW);

		// Check for completely outside
		int32 Mask = VectorAnyGreaterThan(Distance,VRadius);
		if (Mask)
		{
			bOutFullyContained = false;
			return false;
		}

		//the sphere is definitely inside the frustums, but let's check if it's FULLY contained by checking the NEGATIVE radius (on the inside of each frustum plane)
		Mask = VectorAnyGreaterThan(Distance,NegativeVRadius);
		if (Mask)
		{
			bOutFullyContained = false;
		}
	}
	return true;
}

//
//	FConvexVolume::IntersectTriangle
//

bool FConvexVolume::IntersectTriangle(const FVector& PointA, const FVector& PointB, const FVector& PointC, bool& bOutFullyContained) const
{
		checkSlow(PermutedPlanes.Num() % 4 == 0);

		// Assume that it is not fully contained by default
		bOutFullyContained = false;

		// Load the points
		VectorRegister A = VectorLoadFloat3(&PointA);
		VectorRegister B = VectorLoadFloat3(&PointB);
		VectorRegister C = VectorLoadFloat3(&PointC);
		VectorRegister Zero = VectorZero();

		// Splat points into 3 vectors
		VectorRegister AX = VectorReplicate(A, 0);
		VectorRegister AY = VectorReplicate(A, 1);
		VectorRegister AZ = VectorReplicate(A, 2);

		VectorRegister BX = VectorReplicate(B, 0);
		VectorRegister BY = VectorReplicate(B, 1);
		VectorRegister BZ = VectorReplicate(B, 2);

		VectorRegister CX = VectorReplicate(C, 0);
		VectorRegister CY = VectorReplicate(C, 1);
		VectorRegister CZ = VectorReplicate(C, 2);

		// Since we are moving straight through get a pointer to the data
		const FPlane* RESTRICT PermutedPlanePtr = (FPlane*)PermutedPlanes.GetData();

		// First test if one or all point are inside the volume
		bool IsAInside = true;
		bool IsBInside = true;
		bool IsCInside = true;

		// Process four planes at a time until we have < 4 left
		for (int32 Count = 0; Count < PermutedPlanes.Num(); Count += 4)
		{
			// Load 4 planes that are already all Xs, Ys, ...
			VectorRegister PlanesX = VectorLoadAligned(PermutedPlanePtr);
			PermutedPlanePtr++;
			VectorRegister PlanesY = VectorLoadAligned(PermutedPlanePtr);
			PermutedPlanePtr++;
			VectorRegister PlanesZ = VectorLoadAligned(PermutedPlanePtr);
			PermutedPlanePtr++;
			VectorRegister PlanesW = VectorLoadAligned(PermutedPlanePtr);
			PermutedPlanePtr++;

			// Calculate the distance (x * x) + (y * y) + (z * z) - w and use it to determine if the point is ouside

			// Point A
			VectorRegister DistX = VectorMultiply(AX, PlanesX);
			VectorRegister DistY = VectorMultiplyAdd(AY, PlanesY, DistX);
			VectorRegister DistZ = VectorMultiplyAdd(AZ, PlanesZ, DistY);
			VectorRegister Distance = VectorSubtract(DistZ, PlanesW);
			int32 MaskA = VectorAnyGreaterThan(Distance, Zero);
			if (MaskA)
			{
				IsAInside = false;
			}

			DistX = VectorMultiply(BX, PlanesX);
			DistY = VectorMultiplyAdd(BY, PlanesY, DistX);
			DistZ = VectorMultiplyAdd(BZ, PlanesZ, DistY);
			Distance = VectorSubtract(DistZ, PlanesW);
			int32 MaskB = VectorAnyGreaterThan(Distance, Zero);
			if (MaskB)
			{
				IsBInside = false;
			}

			DistX = VectorMultiply(CX, PlanesX);
			DistY = VectorMultiplyAdd(CY, PlanesY, DistX);
			DistZ = VectorMultiplyAdd(CZ, PlanesZ, DistY);
			Distance = VectorSubtract(DistZ, PlanesW);
			int32 MaskC = VectorAnyGreaterThan(Distance, Zero);
			if (MaskC)
			{
				IsCInside = false;
			}

			// The points were all outside of at lest one plane
			if (MaskA & MaskB & MaskC)
			{
				return false;
			}
		}

		// If a point is inside early exit
		if (IsAInside || IsBInside || IsCInside)
		{
			bOutFullyContained = IsAInside && IsBInside && IsCInside;
			return true;
		}

		// Clip the triangle against the planes see if it some part are inside the volume.

		// Arbitrary upper bounds
		const int32 Slack = 16;
		TArray<FVector> Vertices;
		Vertices.Reserve(Slack);
		Vertices.Add(PointA);
		Vertices.Add(PointB);
		Vertices.Add(PointC);

		// True for inside the box
		TArray<bool> Sides;
		Sides.Reserve(Slack);

		TArray<FVector> NewVertices;
		NewVertices.Reserve(Slack);

		for (const FPlane& Plane : Planes)
		{
			bool bVertexInside = false;
			bool bVertexOutside = false;

			VectorRegister PlaneRegist = VectorLoadAligned(&Plane);


			for (int32 Index = 0; Index < Vertices.Num(); ++Index)
			{
				// Calculate the distance of the point and the plane (x * x) + (y * y) + (z * z) - w and use it to determine on which side the point is
				const float DotProduct = VectorDot3Scalar(VectorLoadFloat3(&Vertices[Index]), PlaneRegist);
				if (DotProduct - Plane.W <= 0.0)
				{
					bVertexInside = true;
					Sides.Add(true);
				}
				else
				{
					bVertexOutside = true;
					Sides.Add(false);
				}
			}

			if (!bVertexInside)
			{
				return false;
			}
			
			if (!bVertexOutside)
			{
				continue;
			}


			const int32 LastValidIndex = Vertices.Num() - 1;
			const FVector* PreviousVertex = &Vertices[LastValidIndex];
			bool PreviousIsInside = Sides[LastValidIndex];

			for (int32 Index = 0; Index < Vertices.Num(); ++Index)
			{
				bool bIsInside = Sides[Index];
				const FVector& Vertex = Vertices[Index];
				if (bIsInside != PreviousIsInside)
				{
					// Cross plane
					VectorRegister Start = VectorLoadFloat3(PreviousVertex);
					VectorRegister End = VectorLoadFloat3(&Vertex);
					VectorRegister Line = VectorSubtract(End, Start);

					// Line plane intersection
					// Start + Line * ((W - Dot(Start, PlaneNormal)) / Dot(Line, PlaneNormal))

					// (W - Dot(Start, PlaneNormal)) / Dot(Line, PlaneNormal)
					const FVector::FReal Scalar = (Plane.W - VectorDot3Scalar(Start, PlaneRegist)) / VectorDot3Scalar(Line, PlaneRegist);

					// Start + Line * Scalar
					VectorRegister Intersection = VectorMultiplyAdd(Line, VectorLoadFloat1(&Scalar), Start);

					NewVertices.Emplace(VectorGetComponent(Intersection, 0), VectorGetComponent(Intersection, 1), VectorGetComponent(Intersection, 2));

					if (bIsInside)
					{
						NewVertices.Add(Vertex);
					}
				}
				else if (bIsInside)
				{
					NewVertices.Add(Vertex);
				}

				PreviousIsInside = bIsInside;
				PreviousVertex = &Vertex;
			}


			if (NewVertices.IsEmpty())
			{
				return false;
			}

			Sides.Reset(NewVertices.Num());
			Swap(Vertices, NewVertices);
			NewVertices.Reset();
		}

	return true;
}

//
//	FConvexVolume::IntersectLineSegment
//

bool FConvexVolume::IntersectLineSegment(const FVector& InStart, const FVector& InEnd) const
{
	// @todo: not optimized
	// Not sure if there's a better algorithm for this; in any case, there's scope for vectorizing some of this stuff
	// using the permuted planes array.

	// Take copies of the line segment start/end points so they can be modified
	FVector Start(InStart);
	FVector End(InEnd);

	// Iterate through all planes, successively clipping the line segment against each one,
	// until it is either completely contained within the convex volume (intersects), or
	// it is completely outside (doesn't intersect)
	for (const FPlane& Plane : Planes)
	{
		const float DistanceFromStart = Plane.PlaneDot(Start);
		const float DistanceFromEnd = Plane.PlaneDot(End);

		if (DistanceFromStart > 0.0f && DistanceFromEnd > 0.0f)
		{
			// Both points are outside one of the frustum planes, so cannot intersect
			return false;
		}

		if (DistanceFromStart < 0.0f && DistanceFromEnd < 0.0f)
		{
			// Both points are inside this frustum plane, no need to clip it against the plane
			continue;
		}

		// Clip the line segment against the plane
		const FVector IntersectionPoint = FMath::LinePlaneIntersection(Start, End, Plane);
		if (DistanceFromStart > 0.0f)
		{
			Start = IntersectionPoint;
		}
		else
		{
			End = IntersectionPoint;
		}
	}

	return true;
}


//
//	FConvexVolume::DistanceTo
//

float FConvexVolume::DistanceTo(const FVector& Point) const
{
	checkSlow(PermutedPlanes.Num() % 4 == 0);

	constexpr VectorRegister4Float VMinimumDistance = MakeVectorRegisterFloatConstant(-UE_BIG_NUMBER, -UE_BIG_NUMBER, -UE_BIG_NUMBER, -UE_BIG_NUMBER);

	// Load the origin & radius
	VectorRegister VPoint = VectorLoadFloat3(&Point);
	VectorRegister VMinDistance = VMinimumDistance;
	// Splat point into 3 vectors
	VectorRegister VPointX = VectorReplicate(VPoint, 0);
	VectorRegister VPointY = VectorReplicate(VPoint, 1);
	VectorRegister VPointZ = VectorReplicate(VPoint, 2);
	// Since we are moving straight through get a pointer to the data
	const FPlane* RESTRICT PermutedPlanePtr = (FPlane*)PermutedPlanes.GetData();
	// Process four planes at a time until we have < 4 left
	for (int32 Count = 0; Count < PermutedPlanes.Num(); Count += 4)
	{
		// Load 4 planes that are already all Xs, Ys, ...
		VectorRegister PlanesX = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesY = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesZ = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		VectorRegister PlanesW = VectorLoadAligned(PermutedPlanePtr);
		PermutedPlanePtr++;
		// Calculate the distance (x * x) + (y * y) + (z * z) - w
		VectorRegister DistX = VectorMultiply(VPointX, PlanesX);
		VectorRegister DistY = VectorMultiplyAdd(VPointY, PlanesY, DistX);
		VectorRegister DistZ = VectorMultiplyAdd(VPointZ, PlanesZ, DistY);
		VectorRegister Distance = VectorSubtract(DistZ, PlanesW);

		VMinDistance = VectorMax(Distance, VMinDistance);
	}

	const VectorRegister VMinDistanceWXYZ = VectorSwizzle(VMinDistance, 3, 0, 1, 2);
	const VectorRegister t0 = VectorMax(VMinDistance, VMinDistanceWXYZ);
	const VectorRegister VMinDistanceZWXY = VectorSwizzle(VMinDistance, 2, 3, 0, 1);
	const VectorRegister t1 = VectorMax(t0, VMinDistanceZWXY);
	const VectorRegister VMinDistanceYZWX = VectorSwizzle(VMinDistance, 1, 2, 3, 0);
	const VectorRegister t2 = VectorMax(t1, VMinDistanceYZWX);

	float MinDistance;
	VectorStoreFloat1(t2, &MinDistance);
	return MinDistance;
}

void GetViewFrustumBoundsInternal(FConvexVolume& OutResult, const FMatrix& ViewProjectionMatrix, bool bUseNearPlane, bool bUseFarPlane, const FPlane* InFarPlaneOverride)
{
	OutResult.Planes.Empty(6);
	FPlane Temp;

	// NOTE: Be careful changing anything here! Some callers make assumptions about the order of the planes returned.
	// See for instance BuildLightViewFrustumConvexHull in ShadowSetup.cpp

	// Near clipping plane.
	if (bUseNearPlane && ViewProjectionMatrix.GetFrustumNearPlane(Temp))
	{
		OutResult.Planes.Add(Temp);
	}

	// Left clipping plane.
	if (ViewProjectionMatrix.GetFrustumLeftPlane(Temp))
	{
		OutResult.Planes.Add(Temp);
	}

	// Right clipping plane.
	if (ViewProjectionMatrix.GetFrustumRightPlane(Temp))
	{
		OutResult.Planes.Add(Temp);
	}

	// Top clipping plane.
	if (ViewProjectionMatrix.GetFrustumTopPlane(Temp))
	{
		OutResult.Planes.Add(Temp);
	}

	// Bottom clipping plane.
	if (ViewProjectionMatrix.GetFrustumBottomPlane(Temp))
	{
		OutResult.Planes.Add(Temp);
	}

	// Far clipping plane.
	if (bUseFarPlane)
	{
		if (InFarPlaneOverride != nullptr)
		{
			OutResult.Planes.Add(*InFarPlaneOverride);
		}
		else if (ViewProjectionMatrix.GetFrustumFarPlane(Temp))
		{
			OutResult.Planes.Add(Temp);
		}
	}

	OutResult.Init();
}

void GetViewFrustumBounds(FConvexVolume& OutResult, const FMatrix& ViewProjectionMatrix, bool bUseNearPlane)
{
	GetViewFrustumBoundsInternal(OutResult, ViewProjectionMatrix, bUseNearPlane, true, nullptr);
}

void GetViewFrustumBounds(FConvexVolume& OutResult, const FMatrix& ViewProjectionMatrix, bool bUseNearPlane, bool bUseFarPlane)
{
	GetViewFrustumBoundsInternal(OutResult, ViewProjectionMatrix, bUseNearPlane, bUseFarPlane, nullptr);
}

void GetViewFrustumBounds(FConvexVolume& OutResult, const FMatrix& ViewProjectionMatrix, const FPlane& InFarPlane, bool bOverrideFarPlane, bool bUseNearPlane)
{
	GetViewFrustumBoundsInternal(OutResult, ViewProjectionMatrix, bUseNearPlane, true, bOverrideFarPlane ? &InFarPlane : nullptr);
}

/**
 * Serializor
 *
 * @param	Ar				Archive to serialize data to
 * @param	ConvexVolume	Convex volumes to serialize to archive
 *
 * @return passed in archive
 */
FArchive& operator<<(FArchive& Ar,FConvexVolume& ConvexVolume)
{
	Ar << ConvexVolume.Planes;
	Ar << ConvexVolume.PermutedPlanes;
	return Ar;
}

void DrawFrustumWireframe(
	FPrimitiveDrawInterface* PDI,
	const FMatrix& FrustumToWorld,
	FColor Color,
	uint8 DepthPriority
	)
{
	FVector Vertices[2][2][2];
	for(uint32 Z = 0;Z < 2;Z++)
	{
		for(uint32 Y = 0;Y < 2;Y++)
		{
			for(uint32 X = 0;X < 2;X++)
			{
				FVector4 UnprojectedVertex = FrustumToWorld.TransformFVector4(
					FVector4(
						(X ? -1.0f : 1.0f),
						(Y ? -1.0f : 1.0f),
						(Z ?  0.0f : 1.0f),
						1.0f
						)
					);
				Vertices[X][Y][Z] = FVector(UnprojectedVertex) / UnprojectedVertex.W;
			}
		}
	}

	PDI->DrawLine(Vertices[0][0][0],Vertices[0][0][1],Color,DepthPriority);
	PDI->DrawLine(Vertices[1][0][0],Vertices[1][0][1],Color,DepthPriority);
	PDI->DrawLine(Vertices[0][1][0],Vertices[0][1][1],Color,DepthPriority);
	PDI->DrawLine(Vertices[1][1][0],Vertices[1][1][1],Color,DepthPriority);

	PDI->DrawLine(Vertices[0][0][0],Vertices[0][1][0],Color,DepthPriority);
	PDI->DrawLine(Vertices[1][0][0],Vertices[1][1][0],Color,DepthPriority);
	PDI->DrawLine(Vertices[0][0][1],Vertices[0][1][1],Color,DepthPriority);
	PDI->DrawLine(Vertices[1][0][1],Vertices[1][1][1],Color,DepthPriority);

	PDI->DrawLine(Vertices[0][0][0],Vertices[1][0][0],Color,DepthPriority);
	PDI->DrawLine(Vertices[0][1][0],Vertices[1][1][0],Color,DepthPriority);
	PDI->DrawLine(Vertices[0][0][1],Vertices[1][0][1],Color,DepthPriority);
	PDI->DrawLine(Vertices[0][1][1],Vertices[1][1][1],Color,DepthPriority);
}
