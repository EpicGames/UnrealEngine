// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/CoreSettings.h"
#include "HAL/IConsoleManager.h"
#include "UObject/UnrealType.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CoreSettings)

DEFINE_LOG_CATEGORY_STATIC(LogCoreSettings, Log, All);

int32 GUseUnifiedTimeBudgetForStreaming = 0;
int32 GUseBackgroundLevelStreaming = 1;
float GAsyncLoadingTimeLimit = 5.0f;
int32 GAsyncLoadingUseFullTimeLimit = 0;
float GPriorityAsyncLoadingExtraTime = 15.0f;
float GLevelStreamingActorsUpdateTimeLimit = 5.0f;
float GPriorityLevelStreamingActorsUpdateExtraTime = 5.0f;
float GLevelStreamingUnregisterComponentsTimeLimit = 1.0f;
int32 GLevelStreamingComponentsRegistrationGranularity = 10;
int32 GLevelStreamingAddPrimitiveGranularity = 120;
int32 GLevelStreamingComponentsUnregistrationGranularity = 5;
int32 GLevelStreamingRouteActorInitializationGranularity = 10;
int32 GLevelStreamingRouteActorEndPlayForRemoveFromWorldGranularity = 0;
int32 GLevelStreamingForceGCAfterLevelStreamedOut = 1;
int32 GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge = 1;
int32 GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge = 0; // Disabled by default
int32 GLevelStreamingAllowLevelRequestsWhileAsyncLoadingInMatch = 1;
int32 GLevelStreamingMaxLevelRequestsAtOnceWhileInMatch = 0;
int32 GLevelStreamingForceVerifyLevelsGotRemovedByGC = 0;
int32 GLevelStreamingForceRouteActorInitializeNextFrame = 0;
int32 GLevelStreamingLowMemoryPendingPurgeCount = MAX_int32;
int32 GLevelStreamingLowMemoryActorsPendingPurgeCount = MAX_int32;

static FAutoConsoleVariableRef CVarUseUnifiedTimeBudgetForStreaming(
	TEXT("s.UseUnifiedTimeBudgetForStreaming"),
	GUseUnifiedTimeBudgetForStreaming,
	TEXT("Whether to use unified time budgets and logic for async loading and level streaming.\n")
	TEXT("If 0 (current default), cvars like s.AsyncLoadingTimeLimit and s.LevelStreamingActorsUpdateTimeLimit will separately control each phase and can be modified by game-specific code.\n")
	TEXT("If 1, the time budgets are added together in GetUnifiedTimeBudgetForStreaming, and asset streaming will run immediately before level streaming with a shared budget."),
	ECVF_Default
	);

static FAutoConsoleVariableRef CVarUseBackgroundLevelStreaming(
	TEXT("s.UseBackgroundLevelStreaming"),
	GUseBackgroundLevelStreaming,
	TEXT("Whether to allow background level streaming. If this is 0, all level loading will block the game thread until it completes."),
	ECVF_Default
	);

static FAutoConsoleVariableRef CVarAsyncLoadingTimeLimit(
	TEXT("s.AsyncLoadingTimeLimit"),
	GAsyncLoadingTimeLimit,
	TEXT("Maximum amount of time to spend doing asynchronous asset loading (ms per frame)."),
	ECVF_Default
	);

static FAutoConsoleVariableRef CVarAsyncLoadingUseFullTimeLimit(
	TEXT("s.AsyncLoadingUseFullTimeLimit"),
	GAsyncLoadingUseFullTimeLimit,
	TEXT("Whether to use the entire time limit for async asset loading even if blocked on I/O."),
	ECVF_Default
	);

static FAutoConsoleVariableRef CVarPriorityAsyncLoadingExtraTime(
	TEXT("s.PriorityAsyncLoadingExtraTime"),
	GPriorityAsyncLoadingExtraTime,
	TEXT("Additional time to spend asynchronous loading during a high priority load."),
	ECVF_Default
	);

static FAutoConsoleVariableRef CVarLevelStreamingActorsUpdateTimeLimit(
	TEXT("s.LevelStreamingActorsUpdateTimeLimit"),
	GLevelStreamingActorsUpdateTimeLimit,
	TEXT("Maximum allowed time to spend for actor registration steps during level streaming (ms per frame)."),
	ECVF_Default
	);

static FAutoConsoleVariableRef CVarPriorityLevelStreamingActorsUpdateExtraTime(
	TEXT("s.PriorityLevelStreamingActorsUpdateExtraTime"),
	GPriorityLevelStreamingActorsUpdateExtraTime,
	TEXT("Additional time to spend on actor registration steps during a high priority load."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarLevelStreamingUnregisterComponentsTimeLimit(
	TEXT("s.UnregisterComponentsTimeLimit"),
	GLevelStreamingUnregisterComponentsTimeLimit,
	TEXT("Maximum allowed time to spend for actor unregistration steps during level streaming (ms per frame). If this is zero then we don't timeslice"),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarLevelStreamingComponentsRegistrationGranularity(
	TEXT("s.LevelStreamingComponentsRegistrationGranularity"),
	GLevelStreamingComponentsRegistrationGranularity,
	TEXT("Batching granularity used to register actor components during level streaming."),
	ECVF_Default
	);

static FAutoConsoleVariableRef CVarLevelStreamingAddPrimitiveGranularity(
	TEXT("s.LevelStreamingAddPrimitiveGranularity"),
	GLevelStreamingAddPrimitiveGranularity,
	TEXT("Batching granularity used to add primitives to scene in parallel when registering actor components during level streaming."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarLevelStreamingComponentsUnregistrationGranularity(
	TEXT("s.LevelStreamingComponentsUnregistrationGranularity"),
	GLevelStreamingComponentsUnregistrationGranularity,
	TEXT("Batching granularity used to unregister actor components during level unstreaming."),
	ECVF_Default
	);

static FAutoConsoleVariableRef CVarLevelStreamingRouteActorInitializationGranularity(
	TEXT("s.LevelStreamingRouteActorInitializationGranularity"),
	GLevelStreamingRouteActorInitializationGranularity,
	TEXT("Batching granularity used to initialize actors during level streaming. If this is zero, we process all actors and stages in one pass."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarLevelStreamingRouteActorEndPlayForRemoveFromWorldGranularity(
	TEXT("s.LevelStreamingRouteActorEndPlayForRemoveFromWorldGranularity"),
	GLevelStreamingRouteActorEndPlayForRemoveFromWorldGranularity,
	TEXT("Batching granularity used to call EndPlay on actors when removing streaming level. If this is zero, we process all actors in one pass."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarForceGCAfterLevelStreamedOut(
	TEXT("s.ForceGCAfterLevelStreamedOut"),
	GLevelStreamingForceGCAfterLevelStreamedOut,
	TEXT("Whether to force a GC after levels are streamed out to instantly reclaim the memory at the expensive of a hitch."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarForceVerifyLevelsGotRemovedByGC(
	TEXT("s.ForceVerifyLevelsGotRemovedByGC"),
	GLevelStreamingForceVerifyLevelsGotRemovedByGC,
	TEXT("Whether to force a verification of objects residing in a GC'ed level package (ignored in shipping builds)."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarForceRouteActorInitializeNextFrame(
	TEXT("s.ForceRouteActorInitializeNextFrame"),
	GLevelStreamingForceRouteActorInitializeNextFrame,
	TEXT("Whether to force routing actor initialize phase in its own frame."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarContinuouslyIncrementalGCWhileLevelsPendingPurge(
	TEXT("s.ContinuouslyIncrementalGCWhileLevelsPendingPurge"),
	GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge,
	TEXT("Whether to repeatedly kick off incremental GC when there are levels still waiting to be purged."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarContinuouslyIncrementalGCWhileActorsPendingPurge(
	TEXT("s.ContinuouslyIncrementalGCWhileActorsPendingPurge"),
	GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge,
	TEXT("Whether to repeatedly kick off incremental GC when there are actors (from levels) still waiting to be purged."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarAllowLevelRequestsWhileAsyncLoadingInMatch(
	TEXT("s.AllowLevelRequestsWhileAsyncLoadingInMatch"),
	GLevelStreamingAllowLevelRequestsWhileAsyncLoadingInMatch,
	TEXT("Enables level streaming requests while async loading (of anything) while the match is already in progress and no loading screen is up."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarMaxLevelRequestsAtOnceWhileInMatch(
	TEXT( "s.MaxLevelRequestsAtOnceWhileInMatch" ),
	GLevelStreamingMaxLevelRequestsAtOnceWhileInMatch,
	TEXT( "When we're already loading this many levels and actively in match, don't allow any more requests until one of those completes.  Set to zero to disable." ),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarLevelStreamingLowMemoryPendingPurgeCount(
	TEXT("s.LevelStreamingLowMemoryPendingPurgeCount"),
	GLevelStreamingLowMemoryPendingPurgeCount,
	TEXT("When system is in low memory state, if the number of streaming levels to purge meets or exceeds this value, perform a 'soft' GC."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarLevelStreamingLowMemoryActorsPendingPurgeCount(
	TEXT("s.LevelStreamingLowMemoryActorsPendingPurgeCount"),
	GLevelStreamingLowMemoryActorsPendingPurgeCount,
	TEXT("When system is in low memory state, if the number of actors from streaming levels to purge meets or exceeds this value, perform a 'soft' GC."),
	ECVF_Default
);

UStreamingSettings::UStreamingSettings()
	: Super()
{
	SectionName = TEXT("Streaming");

	AsyncLoadingThreadEnabled = false;
	WarnIfTimeLimitExceeded = false;
	TimeLimitExceededMultiplier = 1.5f;
	TimeLimitExceededMinTime = 0.005f;
	MinBulkDataSizeForAsyncLoading = 131072;
	UseBackgroundLevelStreaming = true;
	AsyncLoadingTimeLimit = 5.0f;
	AsyncLoadingUseFullTimeLimit = true;
	PriorityAsyncLoadingExtraTime = 15.0f;
	LevelStreamingActorsUpdateTimeLimit = 5.0f;
	PriorityLevelStreamingActorsUpdateExtraTime = 5.0f;
	LevelStreamingComponentsRegistrationGranularity = 10;
	LevelStreamingAddPrimitiveGranularity = 120;
	LevelStreamingUnregisterComponentsTimeLimit = 1.0f;
	LevelStreamingComponentsUnregistrationGranularity = 5;
	EventDrivenLoaderEnabled = false;
	FlushStreamingOnExit = true;
}

void UStreamingSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif // #if WITH_EDITOR
}

UGarbageCollectionSettings::UGarbageCollectionSettings()
: Super()
{
	SectionName = TEXT("Garbage Collection");

	TimeBetweenPurgingPendingKillObjects = 60.0f;
	FlushStreamingOnGC = false;
	AllowParallelGC = true;
	IncrementalBeginDestroyEnabled = true;
	NumRetriesBeforeForcingGC = 0;
	MaxObjectsNotConsideredByGC = 0;
	MaxObjectsInEditor = 12 * 1024 * 1024;
	MaxObjectsInGame = 2 * 1024 * 1024;	
	CreateGCClusters = true;	
	MinGCClusterSize = 5;
	AssetClusteringEnabled = true;
	ActorClusteringEnabled = true;
	VerifyUObjectsAreNotFGCObjects = true;
	GarbageEliminationEnabled = false;
	DumpObjectCountsToLogWhenMaxObjectLimitExceeded = false;
}

void UGarbageCollectionSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();

		// Upgrade path for gc.PendingKillEnabled -> gc.GarbageEliminationEnabled
		bool bGarbageEliminationEnabledIni = false;
		if (GConfig->GetBool(TEXT("/Script/Engine.GarbageCollectionSettings"), TEXT("gc.PendingKillEnabled"), bGarbageEliminationEnabledIni, GEngineIni))
		{
			// No need to warn to upgrade to "gc.GarbageEliminationEnabled" as we've already done this in ObjectBaseUtility.cpp InitGarbageElimination()
			GarbageEliminationEnabled = bGarbageEliminationEnabledIni;
		}
	}
#endif // #if WITH_EDITOR
}

#if WITH_EDITOR
void UGarbageCollectionSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}
#endif // #if WITH_EDITOR

