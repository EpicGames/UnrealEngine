// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CubemapUnwapUtils.cpp: Pixel and Vertex shader to render a cube map as 2D texture
=============================================================================*/

#include "CubemapUnwrapUtils.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "GameTime.h"
#include "PipelineStateCache.h"
#include "ShaderParameterUtils.h"
#include "RHIContext.h"
#include "SimpleElementShaders.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureRenderTargetCube.h"
#include "RHIStaticStates.h"
#include "DataDrivenShaderPlatformInfo.h"

IMPLEMENT_SHADER_TYPE(,FCubemapTexturePropertiesVS,TEXT("/Engine/Private/SimpleElementVertexShader.usf"),TEXT("Main"),SF_Vertex);
IMPLEMENT_SHADER_TYPE(,FCubemapTexturePropertiesPS,TEXT("/Engine/Private/SimpleElementPixelShader.usf"),TEXT("CubemapTextureProperties"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FIESLightProfilePS,TEXT("/Engine/Private/SimpleElementPixelShader.usf"),TEXT("IESLightProfileMain"),SF_Pixel);

namespace CubemapHelpers
{
	/**
	* Helper function to create an unwrapped 2D image of the cube map ( longitude/latitude )
	* This version takes explicitly passed properties of the source object, as the sources have different APIs.
	* @param	TextureResource		Source FTextureResource object.
	* @param	AxisDimenion		axis length of the cube.
	* @param	SourcePixelFormat	pixel format of the source.
	* @param	BitsOUT				Raw bits of the 2D image bitmap.
	* @param	SizeXOUT			Filled with the X dimension of the output bitmap.
	* @param	SizeYOUT			Filled with the Y dimension of the output bitmap.
	* @return						true on success.
	* @param	FormatOUT			Filled with the pixel format of the output bitmap.
	*/
	bool GenerateLongLatUnwrap(const FTextureResource* TextureResource, const uint32 AxisDimenion, const EPixelFormat SourcePixelFormat, TArray64<uint8>& BitsOUT, FIntPoint& SizeOUT, EPixelFormat& FormatOUT)
	{
		TRefCountPtr<FBatchedElementParameters> BatchedElementParameters;
		BatchedElementParameters = new FMipLevelBatchedElementParameters((float)0, (float)-1, false, FMatrix44f::Identity, true, true, false);
		const FIntPoint LongLatDimensions(AxisDimenion * 2, AxisDimenion);

		// If the source format is 8 bit per channel or less then select a LDR target format.
		const EPixelFormat TargetPixelFormat = CalculateImageBytes(1, 1, 0, SourcePixelFormat) <= 4 ? PF_B8G8R8A8 : PF_FloatRGBA;

		UTextureRenderTarget2D* RenderTargetLongLat = NewObject<UTextureRenderTarget2D>();
		check(RenderTargetLongLat);
		RenderTargetLongLat->AddToRoot();
		RenderTargetLongLat->ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
		RenderTargetLongLat->InitCustomFormat(LongLatDimensions.X, LongLatDimensions.Y, TargetPixelFormat, false);
		RenderTargetLongLat->TargetGamma = 0;
		FRenderTarget* RenderTarget = RenderTargetLongLat->GameThread_GetRenderTargetResource();

		FCanvas* Canvas = new FCanvas(RenderTarget, NULL, FGameTime(), GMaxRHIFeatureLevel);
		Canvas->SetRenderTarget_GameThread(RenderTarget);

		// Clear the render target to black
		Canvas->Clear(FLinearColor(0, 0, 0, 0));

		FCanvasTileItem TileItem(FVector2D(0.0f, 0.0f), TextureResource, FVector2D(LongLatDimensions.X, LongLatDimensions.Y), FLinearColor::White);
		TileItem.BatchedElementParameters = BatchedElementParameters;
		TileItem.BlendMode = SE_BLEND_Opaque;
		Canvas->DrawItem(TileItem);

		Canvas->Flush_GameThread();
		FlushRenderingCommands();
		Canvas->SetRenderTarget_GameThread(NULL);
		FlushRenderingCommands();
		
		int32 ImageBytes = CalculateImageBytes(LongLatDimensions.X, LongLatDimensions.Y, 0, TargetPixelFormat);

		BitsOUT.AddUninitialized(ImageBytes);

		bool bReadSuccess = false;
		switch (TargetPixelFormat)
		{
			case PF_B8G8R8A8:
				bReadSuccess = RenderTarget->ReadPixelsPtr((FColor*)BitsOUT.GetData());
			break;
			case PF_FloatRGBA:
				{
					TArray<FFloat16Color> FloatColors;
					bReadSuccess = RenderTarget->ReadFloat16Pixels(FloatColors);
					FMemory::Memcpy(BitsOUT.GetData(), FloatColors.GetData(), ImageBytes);
				}
			break;
		}
		// Clean up.
		RenderTargetLongLat->ReleaseResource();
		RenderTargetLongLat->RemoveFromRoot();
		RenderTargetLongLat = NULL;
		delete Canvas;

		SizeOUT = LongLatDimensions;
		FormatOUT = TargetPixelFormat;
		if (bReadSuccess == false)
		{
			// Reading has failed clear output buffer.
			BitsOUT.Empty();
		}

		return bReadSuccess;
	}

	bool GenerateLongLatUnwrap(const UTextureCube* CubeTexture, TArray64<uint8>& BitsOUT, FIntPoint& SizeOUT, EPixelFormat& FormatOUT)
	{
		check(CubeTexture != NULL);
		return GenerateLongLatUnwrap(CubeTexture->GetResource(), CubeTexture->GetSizeX(), CubeTexture->GetPixelFormat(), BitsOUT, SizeOUT, FormatOUT);
	}

	bool GenerateLongLatUnwrap(const UTextureRenderTargetCube* CubeTarget, TArray64<uint8>& BitsOUT, FIntPoint& SizeOUT, EPixelFormat& FormatOUT)
	{
		check(CubeTarget != NULL);
		return GenerateLongLatUnwrap(CubeTarget->GetResource(), CubeTarget->SizeX, CubeTarget->GetFormat(), BitsOUT, SizeOUT, FormatOUT);
	}
}

FCubemapTexturePropertiesVS::FCubemapTexturePropertiesVS() {}

FCubemapTexturePropertiesVS::FCubemapTexturePropertiesVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
	FGlobalShader(Initializer)
{
	Transform.Bind(Initializer.ParameterMap, TEXT("Transform"), SPF_Mandatory);
}

bool FCubemapTexturePropertiesVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsPCPlatform(Parameters.Platform);
}

void FCubemapTexturePropertiesVS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FMatrix& TransformValue )
{
	FMatrix44f TransformValues[2] = { FMatrix44f(TransformValue), FMatrix44f(TransformValue) };
	SetShaderValue(BatchedParameters, Transform, TransformValues);
}

FCubemapTexturePropertiesPS::FCubemapTexturePropertiesPS() = default;

FCubemapTexturePropertiesPS::FCubemapTexturePropertiesPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	CubeTexture.Bind(Initializer.ParameterMap, TEXT("CubeTexture"));
	CubeTextureSampler.Bind(Initializer.ParameterMap, TEXT("CubeTextureSampler"));
	ColorWeights.Bind(Initializer.ParameterMap, TEXT("ColorWeights"));
	PackedProperties0.Bind(Initializer.ParameterMap, TEXT("PackedProperties0"));
	Gamma.Bind(Initializer.ParameterMap, TEXT("Gamma"));
	NumSlices.Bind(Initializer.ParameterMap, TEXT("NumSlices"));
	SliceIndex.Bind(Initializer.ParameterMap, TEXT("SliceIndex"));
	ViewMatrix.Bind(Initializer.ParameterMap, TEXT("ViewMatrix"));
}

bool FCubemapTexturePropertiesPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	if (!IsPCPlatform(Parameters.Platform))
	{
		return false;
	}

	FPermutationDomain PermutationVector(Parameters.PermutationId);
	if (PermutationVector.Get<FCubeArray>() && !IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
	{
		return false;
	}
	return true;
}

void FCubemapTexturePropertiesPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* InTexture, const FMatrix& InColorWeightsValue, float InMipLevel, float InSliceIndex, bool bInIsTextureCubeArray, const FMatrix44f& InViewMatrix, bool bInShowLongLatUnwrap, float InGammaValue, bool bInUsePointSampling)
{
	if (InTexture != nullptr)
	{
		FRHISamplerState* SamplerState = bInUsePointSampling ? TStaticSamplerState<SF_Point>::GetRHI() : InTexture->SamplerStateRHI.GetReference();
		SetTextureParameter(BatchedParameters, CubeTexture, CubeTextureSampler, SamplerState, InTexture->TextureRHI);
	}

	FVector4f PackedProperties0Value(InMipLevel, bInShowLongLatUnwrap ? 1.0f : -1.0f, 0, 0);
	SetShaderValue(BatchedParameters, PackedProperties0, PackedProperties0Value);
	SetShaderValue(BatchedParameters, ColorWeights, (FMatrix44f)InColorWeightsValue);
	SetShaderValue(BatchedParameters, Gamma, InGammaValue);
	SetShaderValue(BatchedParameters, ViewMatrix, InViewMatrix);

	// store slice count and selected slice index for the texture cube array
	if (bInIsTextureCubeArray)
	{
		// GetSizeZ() returns the total number of slices stored in the platform data
		// for a TextureCube array this value is equal to the size of the array multiplied by 6
		const float NumSlicesData = (float)(InTexture != nullptr ? FMath::Max((int32)InTexture->GetSizeZ() / 6, 1) : 1);
		SetShaderValue(BatchedParameters, NumSlices, NumSlicesData);
		SetShaderValue(BatchedParameters, SliceIndex, InSliceIndex);
	}
}

void FMipLevelBatchedElementParameters::BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture)
{
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	TShaderMapRef<FCubemapTexturePropertiesVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));

	typename FCubemapTexturePropertiesPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FCubemapTexturePropertiesPS::FHDROutput>(bHDROutput);
	PermutationVector.Set<FCubemapTexturePropertiesPS::FCubeArray>(bIsTextureCubeArray);

	TShaderMapRef<FCubemapTexturePropertiesPS> PixelShader(GetGlobalShaderMap(InFeatureLevel), PermutationVector);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);
	SetShaderParametersLegacyPS(RHICmdList, PixelShader, Texture, ColorWeights, MipLevel, SliceIndex, bIsTextureCubeArray, ViewMatrix, bShowLongLatUnwrap, InGamma, bUsePointSampling);
}

FIESLightProfilePS::FIESLightProfilePS() = default;

FIESLightProfilePS::FIESLightProfilePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	IESTexture.Bind(Initializer.ParameterMap, TEXT("IESTexture"));
	IESTextureSampler.Bind(Initializer.ParameterMap, TEXT("IESTextureSampler"));
	BrightnessInLumens.Bind(Initializer.ParameterMap, TEXT("BrightnessInLumens"));
}

bool FIESLightProfilePS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && !IsConsolePlatform(Parameters.Platform);
}

void FIESLightProfilePS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, float InBrightnessInLumens )
{
	SetTextureParameter(BatchedParameters, IESTexture, IESTextureSampler, Texture);
	SetShaderValue(BatchedParameters, BrightnessInLumens, InBrightnessInLumens);
}

void FIESLightProfileBatchedElementParameters::BindShaders( FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture )
{
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));
	TShaderMapRef<FIESLightProfilePS> PixelShader(GetGlobalShaderMap(InFeatureLevel));

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);
	SetShaderParametersLegacyPS(RHICmdList, PixelShader, Texture, BrightnessInLumens);
}
