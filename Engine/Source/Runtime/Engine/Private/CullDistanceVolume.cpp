// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/CullDistanceVolume.h"
#include "Engine/CollisionProfile.h"
#include "Engine/World.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CullDistanceVolume)

ACullDistanceVolume::ACullDistanceVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	GetBrushComponent()->bAlwaysCreatePhysicsState = true;

	CullDistances.Add(FCullDistanceSizePair(0,0));
	CullDistances.Add(FCullDistanceSizePair(10000,0));

	bEnabled = true;
}

#if WITH_EDITOR
void ACullDistanceVolume::Destroyed()
{
	Super::Destroyed();

	if (GIsEditor)
	{
		UWorld* World = GetWorld();
		if (World && !World->IsGameWorld())
		{
			World->bDoDelayedUpdateCullDistanceVolumes = true;
		}
	}
}

void ACullDistanceVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	GetWorld()->bDoDelayedUpdateCullDistanceVolumes = true;
}

void ACullDistanceVolume::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);
	if (bFinished)
	{
		GetWorld()->bDoDelayedUpdateCullDistanceVolumes = true;
	}
}
#endif // WITH_EDITOR

bool ACullDistanceVolume::CanBeAffectedByVolumes( UPrimitiveComponent* PrimitiveComponent )
{
	AActor* Owner = PrimitiveComponent ? PrimitiveComponent->GetOwner() : nullptr;

	// Require an owner so we can use its location
	if(	Owner
	// Disregard dynamic actors
	&& (PrimitiveComponent->Mobility == EComponentMobility::Static)
	// Require cull distance volume support to be enabled.
	&&	PrimitiveComponent->bAllowCullDistanceVolume 
	// Skip primitives that is hidden set as we don't want to cull out brush rendering or other helper objects.
	&&	PrimitiveComponent->IsVisible()
	// Disregard prefabs.
	&& !PrimitiveComponent->IsTemplate())
	{
		// Only operate on primitives attached to the owners world.
		UWorld* OwnerWorld = Owner->GetWorld();
		if (OwnerWorld && (PrimitiveComponent->GetScene() == Owner->GetWorld()->Scene))
		{
			return true;
		}
	}
	return false;
}

void ACullDistanceVolume::GetPrimitiveMaxDrawDistances(TMap<UPrimitiveComponent*,float>& OutCullDistances)
{
	// Nothing to do if there is no brush component or no cull distances are set
	if (GetBrushComponent() && CullDistances.Num() > 0 && bEnabled)
	{
		for (TPair<UPrimitiveComponent*, float>& PrimitiveMaxDistancePair : OutCullDistances)
		{
			UPrimitiveComponent* PrimitiveComponent = PrimitiveMaxDistancePair.Key;

			// Check whether primitive supports cull distance volumes and its center point is being encompassed by this volume.
			if (EncompassesPoint(PrimitiveComponent->GetComponentLocation()))
			{		
				// Find best match in CullDistances array.
				const float PrimitiveSize = PrimitiveComponent->Bounds.SphereRadius * 2;
				float CurrentError = FLT_MAX;
				float CurrentCullDistance = 0.f;
				for (const FCullDistanceSizePair& CullDistancePair : CullDistances)
				{
					const float Error = FMath::Abs( PrimitiveSize - CullDistancePair.Size );
					if (Error < CurrentError)
					{
						CurrentError = Error;
						CurrentCullDistance = CullDistancePair.CullDistance;
					}
				}

				float& CullDistance = PrimitiveMaxDistancePair.Value;

				// LD or other volume specified cull distance, use minimum of current and one used for this volume.
				if (CullDistance > 0.f)
				{
					CullDistance = FMath::Min(CullDistance, CurrentCullDistance);
				}
				// LD didn't specify cull distance, use current setting directly.
				else
				{
					CullDistance = CurrentCullDistance;
				}
			}
		}
	}
}

