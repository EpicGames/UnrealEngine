// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/CurveTable.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/Csv/CsvParser.h"
#include "HAL/IConsoleManager.h"
#include "Stats/Stats.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include "EditorFramework/AssetImportData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveTable)

DEFINE_LOG_CATEGORY(LogCurveTable);

DECLARE_CYCLE_STAT(TEXT("CurveTableRowHandle Eval"),STAT_CurveTableRowHandleEval,STATGROUP_Engine);

int32 UCurveTable::GlobalCachedCurveID = 1;

// Enable/Disable optimizing curve tables by removing redundant keys from rows
int32 CVar_CurveTable_RemoveRedundantKeys = 1;
static FAutoConsoleVariableRef CVarCurveTableRemoveRedundantKeys(TEXT("CurveTable.RemoveRedundantKeys"), CVar_CurveTable_RemoveRedundantKeys, TEXT(""), ECVF_Default);


namespace
{
	/** Used to trigger the trigger the data table changed delegate. This allows us to trigger the delegate once from more complex changes */
	struct FScopedCurveTableChange
	{
		FScopedCurveTableChange(UCurveTable* InTable)
			: Table(InTable)
		{
			CriticalSection.Lock();
			int32& Count = ScopeCount.FindOrAdd(Table);
			++Count;
		}
		~FScopedCurveTableChange()
		{
			int32& Count = ScopeCount.FindChecked(Table);
			--Count;
			if (Count == 0)
			{
				Table->OnCurveTableChanged().Broadcast();
				ScopeCount.Remove(Table);
			}

			CriticalSection.Unlock();
		}

	private:
		UCurveTable* Table;

		static TMap<UCurveTable*, int32> ScopeCount;

	public:
		static FTransactionallySafeCriticalSection CriticalSection;
	};

	TMap<UCurveTable*, int32> FScopedCurveTableChange::ScopeCount;
	FTransactionallySafeCriticalSection FScopedCurveTableChange::CriticalSection;

#define CURVETABLE_CHANGE_SCOPE()	FScopedCurveTableChange ActiveScope(this);
}

FTransactionallySafeCriticalSection& UCurveTable::GetCurveTableChangeCriticalSection()
{
	return FScopedCurveTableChange::CriticalSection;
}

//////////////////////////////////////////////////////////////////////////
UCurveTable::UCurveTable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/** Util that removes invalid chars and then make an FName */
FName UCurveTable::MakeValidName(const FString& InString)
{
	FString InvalidChars(INVALID_NAME_CHARACTERS);

	FString FixedString;
	TArray<TCHAR, FString::AllocatorType>& FixedCharArray = FixedString.GetCharArray();

	// Iterate over input string characters
	for (int32 CharIdx=0; CharIdx<InString.Len(); CharIdx++)
	{
		// See if this char occurs in the InvalidChars string
		FString Char = InString.Mid(CharIdx, 1);
		if (!InvalidChars.Contains(Char))
		{
			// Its ok, add to result
			FixedCharArray.Add(Char[0]);
		}
	}
	FixedCharArray.Add(0);

	return FName(*FixedString);
}

void UCurveTable::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar); // When loading, this should load our RowCurve!	

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		CURVETABLE_CHANGE_SCOPE();

		int32 NumRows;
		Ar << NumRows;

		const bool bUpgradingCurveTable = (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ShrinkCurveTableSize);
		if (bUpgradingCurveTable)
		{
			CurveTableMode = (NumRows > 0 ? ECurveTableMode::RichCurves : ECurveTableMode::Empty);
		}
		else
		{
			Ar << CurveTableMode;
		}

		bool bCouldConvertToSimpleCurves = bUpgradingCurveTable;

		// copy any previous curves to free after replacing.
		TMap<FName, FRealCurve*> TempMap(RowMap);

		RowMap.Empty(NumRows);
		for (int32 RowIdx = 0; RowIdx < NumRows; RowIdx++)
		{
			// Load row name
			FName RowName;
			Ar << RowName;

			// Load row data
			if (CurveTableMode == ECurveTableMode::SimpleCurves)
			{
				FSimpleCurve* NewCurve = new FSimpleCurve();
				FSimpleCurve::StaticStruct()->SerializeTaggedProperties(Ar, (uint8*)NewCurve, FSimpleCurve::StaticStruct(), nullptr);

				// Add to map
				RowMap.Add(RowName, NewCurve);
			}
			else
			{
				FRichCurve* NewCurve = new FRichCurve();
				FRichCurve::StaticStruct()->SerializeTaggedProperties(Ar, (uint8*)NewCurve, FRichCurve::StaticStruct(), nullptr);

				// Add to map
				RowMap.Add(RowName, NewCurve);

				if (bCouldConvertToSimpleCurves)
				{
					const TArray<FRichCurveKey>& CurveKeys = NewCurve->GetConstRefOfKeys();
					if (CurveKeys.Num() > 0)
					{
						const ERichCurveInterpMode CommonInterpMode = CurveKeys[0].InterpMode;
						for (const FRichCurveKey& CurveKey : CurveKeys)
						{
							if (CurveKey.InterpMode == RCIM_Cubic || CurveKey.InterpMode != CommonInterpMode)
							{
								bCouldConvertToSimpleCurves = false;
								break;
							}
						}
					}
				}
			}
		}

		if (bCouldConvertToSimpleCurves)
		{
			CurveTableMode = (RowMap.Num() > 0  ? ECurveTableMode::SimpleCurves : ECurveTableMode::Empty);
			for (TPair<FName, FRealCurve*>& Curve : RowMap)
			{
				FSimpleCurve* NewCurve = new FSimpleCurve();
				FRichCurve* OldCurve = (FRichCurve*)Curve.Value;

				const TArray<FRichCurveKey>& CurveKeys = OldCurve->GetConstRefOfKeys();
				if (CurveKeys.Num() > 0)
				{
					NewCurve->SetKeyInterpMode(CurveKeys[0].InterpMode);
					for (const FRichCurveKey& CurveKey : CurveKeys)
					{
						NewCurve->AddKey(CurveKey.Time, CurveKey.Value);
					}
				}

				delete OldCurve;
				Curve.Value = NewCurve;
			}
		}

		for (TPair<FName, FRealCurve*>& Curve : TempMap)
		{
			delete Curve.Value;
		}
	}
	else if (Ar.IsSaving())
	{
		int32 NumRows = RowMap.Num();
		Ar << NumRows;

		Ar << CurveTableMode;

		// Now iterate over rows in the map
		for (auto RowIt = RowMap.CreateIterator(); RowIt; ++RowIt)
		{
			// Save out name
			FName RowName = RowIt.Key();
			Ar << RowName;

			// Save out data
			if (CurveTableMode == ECurveTableMode::SimpleCurves)
			{
				FSimpleCurve* Curve = (FSimpleCurve*)RowIt.Value();
				if (Ar.IsCooking() && Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && !Ar.ShouldSkipBulkData() && CVar_CurveTable_RemoveRedundantKeys > 0)
				{
					Curve->RemoveRedundantKeys(0.f);
				}
				FSimpleCurve::StaticStruct()->SerializeTaggedProperties(Ar, (uint8*)Curve, FSimpleCurve::StaticStruct(), nullptr);
			}
			else
			{
				check(CurveTableMode == ECurveTableMode::RichCurves);
				FRichCurve* Curve = (FRichCurve*)RowIt.Value();
				if (Ar.IsCooking() && Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && !Ar.ShouldSkipBulkData() && CVar_CurveTable_RemoveRedundantKeys > 0)
				{
					Curve->RemoveRedundantAutoTangentKeys(0.f);
				}
				FRichCurve::StaticStruct()->SerializeTaggedProperties(Ar, (uint8*)Curve, FRichCurve::StaticStruct(), nullptr);
			}
		}
	}
	else if (Ar.IsCountingMemory())
	{
		RowMap.CountBytes(Ar);

		if (CurveTableMode == ECurveTableMode::SimpleCurves)
		{
			const size_t SizeOfDirectCurveAllocs = sizeof(FSimpleCurve) * RowMap.Num();
			Ar.CountBytes(SizeOfDirectCurveAllocs, SizeOfDirectCurveAllocs);

			for (const TPair<FName, FRealCurve*>& CurveRow : RowMap)
			{
				FSimpleCurve* Curve = (FSimpleCurve*)CurveRow.Value;
				Curve->Keys.CountBytes(Ar);
			}
		}
		else if (CurveTableMode == ECurveTableMode::RichCurves)
		{
			const size_t SizeOfDirectCurveAllocs = sizeof(FRichCurve) * RowMap.Num();
			Ar.CountBytes(SizeOfDirectCurveAllocs, SizeOfDirectCurveAllocs);

			for (const TPair<FName, FRealCurve*>& CurveRow : RowMap)
			{
				FRichCurve* Curve = (FRichCurve*)CurveRow.Value;
				Curve->Keys.CountBytes(Ar);
			}
		}
	}
}

void UCurveTable::FinishDestroy()
{
	CURVETABLE_CHANGE_SCOPE();

	Super::FinishDestroy();

	EmptyTable(); // Free memory when UObject goes away
}

#if WITH_EDITORONLY_DATA
void UCurveTable::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}

	Super::GetAssetRegistryTags(Context);
}

void UCurveTable::PostInitProperties()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}

	Super::PostInitProperties();
}

void UCurveTable::PostLoad()
{
	Super::PostLoad();
	if (!ImportPath_DEPRECATED.IsEmpty() && AssetImportData)
	{
		FAssetImportInfo Info;
		Info.Insert(FAssetImportInfo::FSourceFile(ImportPath_DEPRECATED));
		AssetImportData->SourceData = MoveTemp(Info);
	}
}
#endif

template<class T>
void GetTableAsString_Internal(const TMap<FName, T*>& RowMap, FString& Result)
{
	TArray<FName> Names;
	TArray<T*> Curves;

	// get the row names and curves they represent
	RowMap.GenerateKeyArray(Names);
	RowMap.GenerateValueArray(Curves);

	// Determine the curve with the longest set of data, for headers
	int32 LongestCurveIndex = 0;
	for (int32 CurvesIdx = 1; CurvesIdx < Curves.Num(); CurvesIdx++)
	{
		if (Curves[CurvesIdx]->GetNumKeys() > Curves[LongestCurveIndex]->GetNumKeys())
		{
			LongestCurveIndex = CurvesIdx;
		}
	}

	// First row, column titles, taken from the longest curve
	Result += TEXT("---");
	for (auto It(Curves[LongestCurveIndex]->GetKeyIterator()); It; ++It)
	{
		Result += FString::Printf(TEXT(",%f"), It->Time);
	}
	Result += TEXT("\n");

	// display all the curves
	for (int32 CurvesIdx = 0; CurvesIdx < Curves.Num(); CurvesIdx++)
	{
		// show name of curve
		Result += Names[CurvesIdx].ToString();

		// show data of curve
		for (auto It(Curves[CurvesIdx]->GetKeyIterator()); It; ++It)
		{
			Result += FString::Printf(TEXT(",%f"), It->Value);
		}

		Result += TEXT("\n");
	}
}

FString UCurveTable::GetTableAsString() const
{
	FString Result;

	if (RowMap.Num() > 0)
	{
		if (CurveTableMode == ECurveTableMode::SimpleCurves)
		{
			GetTableAsString_Internal<FSimpleCurve>(*reinterpret_cast<const TMap<FName, FSimpleCurve*>*>(&RowMap), Result);
		}
		else
		{
			GetTableAsString_Internal<FRichCurve>(*reinterpret_cast<const TMap<FName, FRichCurve*>*>(&RowMap), Result);
		}
	}
	else
	{
		Result += FString("No data in row curve!\n");
	}
	return Result;
}

template<class T>
void GetTableAsCSV_Internal(const TMap<FName, T*>& RowMap, FString& Result)
{
	TArray<FName> Names;
	TArray<T*> Curves;

	// get the row names and curves they represent
	RowMap.GenerateKeyArray(Names);
	RowMap.GenerateValueArray(Curves);

	// Determine the curve with the longest set of data, for headers
	int32 LongestCurveIndex = 0;
	for (int32 CurvesIdx = 1; CurvesIdx < Curves.Num(); CurvesIdx++)
	{
		if (Curves[CurvesIdx]->GetNumKeys() > Curves[LongestCurveIndex]->GetNumKeys())
		{
			LongestCurveIndex = CurvesIdx;
		}
	}

	// First row, column titles, taken from the longest curve
	Result += TEXT("---");
	for (auto It(Curves[LongestCurveIndex]->GetKeyIterator()); It; ++It)
	{
		Result += FString::Printf(TEXT(",%f"), It->Time);
	}
	Result += TEXT("\n");

	// display all the curves
	for (int32 CurvesIdx = 0; CurvesIdx < Curves.Num(); CurvesIdx++)
	{
		// show name of curve
		Result += Names[CurvesIdx].ToString();

		// show data of curve
		for (auto It(Curves[CurvesIdx]->GetKeyIterator()); It; ++It)
		{
			Result += FString::Printf(TEXT(",%f"), It->Value);
		}

		Result += TEXT("\n");
	}
}

FString UCurveTable::GetTableAsCSV() const
{
	FString Result;

	if (RowMap.Num() > 0)
	{
		if (CurveTableMode == ECurveTableMode::SimpleCurves)
		{
			GetTableAsCSV_Internal<FSimpleCurve>(*reinterpret_cast<const TMap<FName, FSimpleCurve*>*>(&RowMap), Result);
		}
		else
		{
			GetTableAsCSV_Internal<FRichCurve>(*reinterpret_cast<const TMap<FName, FRichCurve*>*>(&RowMap), Result);
		}
	}

	return Result;
}

FString UCurveTable::GetTableAsJSON() const
{
	// use the pretty print policy since these values are usually getting dumpped for check-in to P4 (or for inspection)
	FString Result;
	TSharedRef< TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&Result);
	if (!WriteTableAsJSON(JsonWriter))
	{
		return TEXT("No data in row curve!\n");
	}
	JsonWriter->Close();
	return Result;
}

template<typename CharType, class T>
void WriteTableAsJSON_Internal(const TMap<FName, T*>& RowMap, const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter, bool bAsArray)
{
	TArray<FName> Names;
	TArray<T*> Curves;

	// get the row names and curves they represent
	RowMap.GenerateKeyArray(Names);
	RowMap.GenerateValueArray(Curves);

	// Determine the curve with the longest set of data, for headers
	int32 LongestCurveIndex = 0;
	for (int32 CurvesIdx = 1; CurvesIdx < Curves.Num(); CurvesIdx++)
	{
		if (Curves[CurvesIdx]->GetNumKeys() > Curves[LongestCurveIndex]->GetNumKeys())
		{
			LongestCurveIndex = CurvesIdx;
		}
	}

	if (bAsArray)
	{
		JsonWriter->WriteArrayStart();
	}

	// display all the curves
	for (int32 CurvesIdx = 0; CurvesIdx < Curves.Num(); CurvesIdx++)
	{
		if (bAsArray)
		{
			JsonWriter->WriteObjectStart();
			// show name of curve
			JsonWriter->WriteValue(TEXT("Name"), Names[CurvesIdx].ToString());
		}
		else
		{
			JsonWriter->WriteObjectStart(Names[CurvesIdx].ToString());
		}
		// show data of curve
		auto LongIt(Curves[LongestCurveIndex]->GetKeyIterator());
		for (auto It(Curves[CurvesIdx]->GetKeyIterator()); It; ++It)
		{
			JsonWriter->WriteValue(FString::SanitizeFloat(LongIt->Time, 0), It->Value);
			++LongIt;
		}
		JsonWriter->WriteObjectEnd();
	}

	if (bAsArray)
	{
		JsonWriter->WriteArrayEnd();
	}
}

template <typename CharType>
bool UCurveTable::WriteTableAsJSON(const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter, bool bAsArray) const
{
	if (RowMap.Num() <= 0)
	{
		return false;
	}

	if (CurveTableMode == ECurveTableMode::SimpleCurves)
	{
		WriteTableAsJSON_Internal<CharType, FSimpleCurve>(*reinterpret_cast<const TMap<FName, FSimpleCurve*>*>(&RowMap), JsonWriter, bAsArray);
	}
	else
	{
		WriteTableAsJSON_Internal<CharType, FRichCurve>(*reinterpret_cast<const TMap<FName, FRichCurve*>*>(&RowMap), JsonWriter, bAsArray);
	}

	return true;
}

template ENGINE_API bool UCurveTable::WriteTableAsJSON<TCHAR>(const TSharedRef< TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR> > >& JsonWriter, bool bAsArray) const;
template ENGINE_API bool UCurveTable::WriteTableAsJSON<ANSICHAR>(const TSharedRef< TJsonWriter<ANSICHAR, TPrettyJsonPrintPolicy<ANSICHAR> > >& JsonWriter, bool bAsArray) const;

void UCurveTable::EmptyTable()
{
	CURVETABLE_CHANGE_SCOPE();

	if (RowMap.Num() > 0)
	{
		for (const TPair<FName, FRealCurve*>& CurveRow : RowMap)
		{
			delete CurveRow.Value;
		}

		// Finally empty the map
		RowMap.Empty();

		if (HasAnyFlags(RF_LoadCompleted) || !HasAnyFlags(RF_WasLoaded))
		{
			// AttributeSets can cache pointers to curves in this table, so we'll need to make sure they've
			// all been invalidated properly, since we just blew them away.
			// Ignore this for objects that are still loading as nothing can reference them yet.
			UCurveTable::InvalidateAllCachedCurves();
		}
	}

	CurveTableMode = ECurveTableMode::Empty;
}

void UCurveTable::RemoveRow(FName RowName)
{
	FRealCurve* Curve = nullptr;
	RowMap.RemoveAndCopyValue(RowName, Curve);
	if (Curve != nullptr)
	{
		delete Curve;
	}
}

FRichCurve& UCurveTable::AddRichCurve(FName RowName)
{
	check(CurveTableMode != ECurveTableMode::SimpleCurves);
	CurveTableMode = ECurveTableMode::RichCurves;

	FRichCurve* Result = new FRichCurve();
	if (FRealCurve** Curve = RowMap.Find(RowName))
	{
		delete *Curve;
		*Curve = Result;
	}
	else
	{
		RowMap.Add(RowName, Result);
	}

	return *Result; 
}

FSimpleCurve& UCurveTable::AddSimpleCurve(FName RowName)
{
	check(CurveTableMode != ECurveTableMode::RichCurves);
	CurveTableMode = ECurveTableMode::SimpleCurves;

	FSimpleCurve* Result = new FSimpleCurve();
	if (FRealCurve** Curve = RowMap.Find(RowName))
	{
		delete *Curve;
		*Curve = Result;
	}
	else
	{
		RowMap.Add(RowName, Result);
	}

	return *Result;
}

void UCurveTable::RenameRow(const FName& CurveName, const FName& NewCurveName)
{
	if (CurveName != NewCurveName && !RowMap.Contains(NewCurveName))
	{
		if (FRealCurve** Curve = RowMap.Find(CurveName))
		{
			RowMap.Add(NewCurveName, *Curve);
			RowMap.Remove(CurveName);
		}
	}
}

void UCurveTable::DeleteRow(const FName& CurveName)
{
	FRealCurve* CurveToDelete = nullptr;
	if (RowMap.RemoveAndCopyValue(CurveName, CurveToDelete))
	{
		delete CurveToDelete;
	}
}

/** */
void GetCurveValues(const TArray<const TCHAR*>& Cells, TArray<float>& Values)
{
	// Need at least 2 columns, first column is skipped, will contain row names
	if (Cells.Num() >= 2)
	{
		Values.Reserve(Values.Num() + Cells.Num() - 1);
		// first element always NULL - as first column is row names
		for (int32 ColIdx = 1; ColIdx < Cells.Num(); ColIdx++)
		{
			Values.Add(FCString::Atof(Cells[ColIdx]));
		}
	}
}

bool FindDuplicateXValues(const TArray<float>& XValues, const FString& ContextString, TArray<FString>& OutProblems)
{
	bool bDoesContainDuplicates = false;
	int32 NumColumns = XValues.Num();
	for (int32 ColIdx = 0; ColIdx < NumColumns; ++ColIdx)
	{
		for (int32 InnerIdx = ColIdx + 1; InnerIdx < NumColumns; ++InnerIdx)
		{
			if (XValues[ColIdx] == XValues[InnerIdx])
			{
				bDoesContainDuplicates = true;
				OutProblems.Add(FString::Printf(TEXT("Found duplicate columns in %s. %f is used in columns %d and %d"), *ContextString, XValues[ColIdx], ColIdx, InnerIdx));
			}
		}
	}

	return bDoesContainDuplicates;
}

TArray<FString> UCurveTable::CreateTableFromCSVString(const FString& InString, ERichCurveInterpMode InterpMode)
{
	CURVETABLE_CHANGE_SCOPE();

	// Array used to store problems about table creation
	TArray<FString> OutProblems;

	const FCsvParser Parser(InString);
	const FCsvParser::FRows& Rows = Parser.GetRows();

	// Must have at least 2 rows (x values + y values for at least one row)
	if (Rows.Num() <= 1)
	{
		OutProblems.Add(FString("Too few rows."));
		return OutProblems;
	}

	// Empty existing data
	EmptyTable();

	CurveTableMode = (InterpMode != RCIM_Cubic ? ECurveTableMode::SimpleCurves : ECurveTableMode::RichCurves);

#if WITH_EDITORONLY_DATA
	ImportCurveInterpMode = InterpMode;
#endif // WITH_EDITORONLY_DATA

	TArray<float> XValues;
	GetCurveValues(Rows[0], XValues);

	// check for duplicate Column values
	if (FindDuplicateXValues(XValues, TEXT("UCurveTable::CreateTableFromCSVString"), OutProblems))
	{
		return OutProblems;
	}

	// Iterate over rows
	for (int32 RowIdx = 1; RowIdx < Rows.Num(); RowIdx++)
	{
		const TArray<const TCHAR*>& Row = Rows[RowIdx];

		// Need at least 1 cells (row name)
		if (Row.Num() < 1)
		{
			OutProblems.Add(FString::Printf(TEXT("Row '%d' has too few cells."), RowIdx));
			continue;
		}

		// Get row name
		FName RowName = MakeValidName(Row[0]);

		// Check its not 'none'
		if (RowName == NAME_None)
		{
			OutProblems.Add(FString::Printf(TEXT("Row '%d' missing a name."), RowIdx));
			continue;
		}

		// Check its not a duplicate
		if (RowMap.Find(RowName) != nullptr)
		{
			OutProblems.Add(FString::Printf(TEXT("Duplicate row name '%s'."), *RowName.ToString()));
			continue;
		}

		TArray<float> YValues;
		GetCurveValues(Row, YValues);

		if (XValues.Num() != YValues.Num())
		{
			OutProblems.Add(FString::Printf(TEXT("Row '%s' does not have the right number of columns."), *RowName.ToString()));
			continue;
		}

		if (CurveTableMode == ECurveTableMode::SimpleCurves)
		{
			FSimpleCurve* NewCurve = new FSimpleCurve();
			NewCurve->SetKeyInterpMode(InterpMode);
			// Now iterate over cells (skipping first cell, that was row name)
			for (int32 ColumnIdx = 0; ColumnIdx < XValues.Num(); ColumnIdx++)
			{
				NewCurve->AddKey(XValues[ColumnIdx], YValues[ColumnIdx]);
			}

			RowMap.Add(RowName, NewCurve);
		}
		else
		{
			FRichCurve* NewCurve = new FRichCurve();
			// Now iterate over cells (skipping first cell, that was row name)
			for (int32 ColumnIdx = 0; ColumnIdx < XValues.Num(); ColumnIdx++)
			{
				FKeyHandle KeyHandle = NewCurve->AddKey(XValues[ColumnIdx], YValues[ColumnIdx]);
				NewCurve->SetKeyInterpMode(KeyHandle, InterpMode);
			}

			RowMap.Add(RowName, NewCurve);
		}
	}

	OnCurveTableChanged().Broadcast();

	Modify(true);

	return OutProblems;
}

template<class CurveType>
void CopySharedInterpolationMode(const CurveType* SourceCurve, CurveType* TargetCurve) { }

template<>
void CopySharedInterpolationMode<FSimpleCurve>(const FSimpleCurve* SourceCurve, FSimpleCurve* TargetCurve)
{
	if (SourceCurve && TargetCurve)
	{
		TargetCurve->SetKeyInterpMode(SourceCurve->GetKeyInterpMode());
	}
}

template<class CurveType, class CurveKeyType>
void CopyRowsToTable(const TMap<FName, CurveType*>& SourceRows, TMap<FName, FRealCurve*>& RowMap, TArray<FString>& OutProblems)
{
	for (const TPair<FName, CurveType*>& CurveRow : SourceRows)
	{
		CurveType* NewCurve = new CurveType();
		CurveType* InCurve = CurveRow.Value;
		const TArray<CurveKeyType>& CurveKeys = InCurve->GetConstRefOfKeys();
		NewCurve->SetKeys(CurveKeys);
		
		// Simple curves need to setup their InterpMode
		CopySharedInterpolationMode(InCurve, NewCurve);

#if WITH_EDITOR
		// check for duplicate key entries
		TArray<float> XValues;
		XValues.Reserve(CurveKeys.Num());
		for (const CurveKeyType& Key : CurveKeys)
		{
			XValues.Add(Key.Time);
		}
		FString ContextString = FString::Printf(TEXT("UCurveTable::CreateTableFromOtherTable (row=%s)"), *CurveRow.Key.ToString());

		if (FindDuplicateXValues(XValues, ContextString, OutProblems))
		{
			continue;
		}
#endif

		RowMap.Add(CurveRow.Key, NewCurve);
	}
}

TArray<FString> UCurveTable::CreateTableFromOtherTable(const UCurveTable* InTable)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UCurveTable::CreateTableFromOtherTable);

	CURVETABLE_CHANGE_SCOPE();

	// Array used to store problems about table creation
	TArray<FString> OutProblems;

	if (InTable == nullptr)
	{
		OutProblems.Add(TEXT("No input table provided"));
		return OutProblems;
	}

	const bool bUseSimpleCurves = (InTable->CurveTableMode == ECurveTableMode::SimpleCurves);

	if (bUseSimpleCurves)
	{
		// make a local copy of the rowmap so we have a snapshot of it
		TMap<FName, FSimpleCurve*> InRowMapCopy = InTable->GetSimpleCurveRowMap();
		EmptyTable();
		CopyRowsToTable<FSimpleCurve, FSimpleCurveKey>(InRowMapCopy, RowMap, OutProblems);
	}
	else
	{
		// make a local copy of the rowmap so we have a snapshot of it
		TMap<FName, FRichCurve*> InRowMapCopy = InTable->GetRichCurveRowMap();
		EmptyTable();
		CopyRowsToTable<FRichCurve, FRichCurveKey>(InRowMapCopy, RowMap, OutProblems);
	}

	CurveTableMode = InTable->CurveTableMode;

	// This is already called when getting out of scope because of CURVETABLE_CHANGE_SCOPE() above.
	// OnCurveTableChanged().Broadcast();

	return OutProblems;
}

TArray<FString> UCurveTable::CreateTableFromJSONString(const FString& InString, ERichCurveInterpMode InterpMode)
{
	CURVETABLE_CHANGE_SCOPE();

	// Array used to store problems about table creation
	TArray<FString> OutProblems;

	if (InString.IsEmpty())
	{
		OutProblems.Add(TEXT("Input data is empty."));
		return OutProblems;
	}

	TArray< TSharedPtr<FJsonValue> > ParsedTableRows;
	{
		const TSharedRef< TJsonReader<TCHAR> > JsonReader = TJsonReaderFactory<TCHAR>::Create(InString);
		if (!FJsonSerializer::Deserialize(JsonReader, ParsedTableRows) || ParsedTableRows.Num() == 0)
		{
			OutProblems.Add(FString::Printf(TEXT("Failed to parse the JSON data. Error: %s"), *JsonReader->GetErrorMessage()));
			return OutProblems;
		}
	}

	// Empty existing data
	EmptyTable();

	CurveTableMode = (InterpMode != RCIM_Cubic ? ECurveTableMode::SimpleCurves : ECurveTableMode::RichCurves);

#if WITH_EDITORONLY_DATA
	ImportCurveInterpMode = InterpMode;
#endif // WITH_EDITORONLY_DATA

	// Iterate over rows
	for (int32 RowIdx = 0; RowIdx < ParsedTableRows.Num(); ++RowIdx)
	{
		const TSharedPtr<FJsonValue>& ParsedTableRowValue = ParsedTableRows[RowIdx];
		TSharedPtr<FJsonObject> ParsedTableRowObject = ParsedTableRowValue->AsObject();
		if (!ParsedTableRowObject.IsValid())
		{
			OutProblems.Add(FString::Printf(TEXT("Row '%d' is not a valid JSON object."), RowIdx));
			continue;
		}

		// Get row name
		static const FString RowNameJsonKey = "Name";
		const FName RowName = MakeValidName(ParsedTableRowObject->GetStringField(RowNameJsonKey));

		// Check its not 'none'
		if (RowName == NAME_None)
		{
			OutProblems.Add(FString::Printf(TEXT("Row '%d' missing a name."), RowIdx));
			continue;
		}

		// Check its not a duplicate
		if (RowMap.Find(RowName) != nullptr)
		{
			OutProblems.Add(FString::Printf(TEXT("Duplicate row name '%s'."), *RowName.ToString()));
			continue;
		}

		// Add a key for each entry in this row
		FSimpleCurve* NewSimpleCurve = nullptr;
		FRichCurve* NewRichCurve = nullptr;

		if (CurveTableMode == ECurveTableMode::SimpleCurves)
		{
			NewSimpleCurve = new FSimpleCurve();
			NewSimpleCurve->SetKeyInterpMode(InterpMode);
		}
		else
		{
			NewRichCurve = new FRichCurve();
		}

		for (const auto& ParsedTableRowEntry : ParsedTableRowObject->Values)
		{
			// Skip the name entry
			if (FStringView(ParsedTableRowEntry.Key) == RowNameJsonKey)
			{
				continue;
			}

			// Make sure we have a valid float key
			float EntryKey = 0.0f;
			if (!LexTryParseString(EntryKey, *ParsedTableRowEntry.Key))
			{
				OutProblems.Add(FString::Printf(TEXT("Key '%s' on row '%s' is not a float and cannot be parsed."), *ParsedTableRowEntry.Key, *RowName.ToString()));
				continue;
			}

			// Make sure we have a valid float value
			double EntryValue = 0.0;
			if (!ParsedTableRowEntry.Value->TryGetNumber(EntryValue))
			{
				OutProblems.Add(FString::Printf(TEXT("Entry '%s' on row '%s' is not a float and cannot be parsed."), *ParsedTableRowEntry.Key, *RowName.ToString()));
				continue;
			}

			if (NewSimpleCurve)
			{
				NewSimpleCurve->AddKey(EntryKey, static_cast<float>(EntryValue));
			}
			else
			{
				checkSlow(NewRichCurve);
				FKeyHandle KeyHandle = NewRichCurve->AddKey(EntryKey, static_cast<float>(EntryValue));
				NewRichCurve->SetKeyInterpMode(KeyHandle, InterpMode);
			}
		}

		// check for duplicate key entries
		TArray<float> XValues;
		if (NewSimpleCurve)
		{
			const TArray<FSimpleCurveKey>& CurveKeys = NewSimpleCurve->GetConstRefOfKeys();
			XValues.Reserve(CurveKeys.Num());
			for (const FSimpleCurveKey& Key : CurveKeys)
			{
				XValues.Add(Key.Time);
			}
		}
		else
		{
			checkSlow(NewRichCurve);
			const TArray<FRichCurveKey>& CurveKeys = NewRichCurve->GetConstRefOfKeys();
			XValues.Reserve(CurveKeys.Num());
			for (const FRichCurveKey& Key : CurveKeys)
			{
				XValues.Add(Key.Time);
			}

		}

		FString ContextString = FString::Printf(TEXT("UCurveTable::CreateTableFromJSONString (row=%s)"), *RowName.ToString());
		if (FindDuplicateXValues(XValues, ContextString, OutProblems))
		{
			continue;
		}

		if (NewSimpleCurve)
		{
			RowMap.Add(RowName, NewSimpleCurve);
		}
		else
		{
			checkSlow(NewRichCurve);
			RowMap.Add(RowName, NewRichCurve);
		}
	}

	OnCurveTableChanged().Broadcast();

	Modify(true);

	return OutProblems;
}

TArray<FRichCurveEditInfoConst> UCurveTable::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;

	for (auto Iter = RowMap.CreateConstIterator(); Iter; ++Iter)
	{
		Curves.Add(FRichCurveEditInfoConst(Iter.Value(), Iter.Key()));
	}

	return Curves;
}

void UCurveTable::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	for (auto Iter = RowMap.CreateConstIterator(); Iter; ++Iter)
	{
		Curves.Add(FRichCurveEditInfoConst(Iter.Value(), Iter.Key()));
	}
}

TArray<FRichCurveEditInfo> UCurveTable::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	
	for (auto Iter = RowMap.CreateIterator(); Iter; ++Iter)
	{
		Curves.Add(FRichCurveEditInfo(Iter.Value(), Iter.Key()));
	}

	return Curves;
}

void UCurveTable::ModifyOwner()
{
	Modify(true);
}

void UCurveTable::MakeTransactional()
{
	SetFlags(GetFlags() | RF_Transactional);
}

void UCurveTable::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	CURVETABLE_CHANGE_SCOPE();
	OnCurveTableChanged().Broadcast();
}

bool UCurveTable::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	for (auto Iter = RowMap.CreateConstIterator(); Iter; ++Iter)
	{
		if (CurveInfo.CurveToEdit == Iter.Value())
		{
			return true;
		}
	}

	return false;
}

void UCurveTable::InvalidateAllCachedCurves()
{
	GlobalCachedCurveID++;
}


TArray<const UObject*> UCurveTable::GetOwners() const
{
	TArray<const UObject*> Owners;
	Owners.Add(this);

	return Owners;
}

#if WITH_EDITOR
void UCurveTable::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnCurveTableChanged().Broadcast();
}
#endif


//////////////////////////////////////////////////////////////////////////

FRealCurve* FCurveTableRowHandle::GetCurve(const FString& ContextString, bool bWarnIfNotFound) const
{
	if (CurveTable == nullptr)
	{
		if (RowName != NAME_None)
		{
			UE_CLOGF(bWarnIfNotFound, LogCurveTable, Warning, "FCurveTableRowHandle::FindRow : No CurveTable for row %ls (%ls).", *RowName.ToString(), *ContextString);
		}
		return nullptr;
	}

	return CurveTable->FindCurve(RowName, ContextString, bWarnIfNotFound);
}

FRichCurve* FCurveTableRowHandle::GetRichCurve(const FString& ContextString, bool bWarnIfNotFound) const
{
	if (CurveTable == nullptr)
	{
		if (RowName != NAME_None)
		{
			UE_CLOGF(bWarnIfNotFound, LogCurveTable, Warning, "FCurveTableRowHandle::FindRow : No CurveTable for row %ls (%ls).", *RowName.ToString(), *ContextString);
		}
		return nullptr;
	}

	return CurveTable->FindRichCurve(RowName, ContextString, bWarnIfNotFound);
}

FSimpleCurve* FCurveTableRowHandle::GetSimpleCurve(const FString& ContextString, bool bWarnIfNotFound) const
{
	if (CurveTable == nullptr)
	{
		if (RowName != NAME_None)
		{
			UE_CLOGF(bWarnIfNotFound, LogCurveTable, Warning, "FCurveTableRowHandle::FindRow : No CurveTable for row %ls (%ls).", *RowName.ToString(), *ContextString);
		}
		return nullptr;
	}

	return CurveTable->FindSimpleCurve(RowName, ContextString, bWarnIfNotFound);
}

bool FCurveTableRowHandle::Eval(float XValue, float* YValue, const FString& ContextString) const
{
	SCOPE_CYCLE_COUNTER(STAT_CurveTableRowHandleEval); 

	FRealCurve* Curve = GetCurve(ContextString);
	if (Curve != nullptr && YValue != nullptr)
	{
		*YValue = Curve->Eval(XValue);
		return true;
	}

	return false;
}

bool FCurveTableRowHandle::operator==(const FCurveTableRowHandle& Other) const
{
	return ((Other.CurveTable == CurveTable) && (Other.RowName == RowName));
}

bool FCurveTableRowHandle::operator!=(const FCurveTableRowHandle& Other) const
{
	return ((Other.CurveTable != CurveTable) || (Other.RowName != RowName));
}
void FCurveTableRowHandle::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsSaving() && !IsNull() && CurveTable)
	{
		// Note which row we are pointing to for later searching
		Ar.MarkSearchableName(CurveTable, RowName);
	}
}

