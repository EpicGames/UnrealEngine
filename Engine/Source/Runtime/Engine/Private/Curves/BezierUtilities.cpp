// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/BezierUtilities.h"

namespace UE
{
namespace CubicBezier
{

void SplitAt(const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, const float t, FVector OutResult[5])
{
	// Split using De Casteljau algorithm.
	const FVector P01 = FMath::Lerp(P0, P1, t);
	const FVector P12 = FMath::Lerp(P1, P2, t);
	const FVector P23 = FMath::Lerp(P2, P3, t);

	OutResult[0] = P01;
	OutResult[1] = FMath::Lerp(P01, P12, t);

	OutResult[3] = FMath::Lerp(P12, P23, t);
	OutResult[4] = P23;

	OutResult[2] = FMath::Lerp(OutResult[1], OutResult[3], t);
}

FBox CalcBounds(const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3)
{
	FBox Result(EForceInit::ForceInit);

	// Start the bounding box by end points
	Result += P0;
	Result += P3;

	// Bezier curve fits inside the convex hull of it's control points.
	// If control points are inside the bounds, we're done.
	if (Result.IsInsideOrOn(P1) && Result.IsInsideOrOn(P2))
		return Result;

	// Add bezier curve inflection points in X, Y, and Z.
	const FVector A = -3.0f * P0 + 9.0f * P1 - 9.0f * P2 + 3.0f * P3;
	const FVector B = 6.0f * P0 - 12.0f * P1 + 6.0f * P2;
	const FVector C = 3.0f * P1 - 3.0f * P0;

	for (int32 Axis = 0; Axis < 3; Axis++)
	{
		float Roots[2];
		int32 RootNum = 0;
		if (FMath::IsNearlyZero(A[Axis]))
		{
			// It's linear equation.
			if (!FMath::IsNearlyZero(B[Axis]))
			{
				Roots[0] = -C[Axis] / B[Axis];
				RootNum = 1;
			}
		}
		else
		{
			// Solve quadratic roots.
			float Disc = B[Axis] * B[Axis] - 4.0f * C[Axis] * A[Axis];
			if (Disc > UE_KINDA_SMALL_NUMBER)
			{
				Disc = FMath::Sqrt(Disc);
				const float Denom = 1.0f / (2.0f * A[Axis]);
				Roots[0] = (-B[Axis] + Disc) * Denom;
				Roots[1] = (-B[Axis] - Disc) * Denom;
				RootNum = 2;
			}
		}
		for (int32 i = 0; i < RootNum; i++)
		{
			// Accept only roots which are inside the bounds of the curve (also excluding enpoints which are already added above).
			const float t = Roots[i];
			if (t > UE_KINDA_SMALL_NUMBER && t < (1.0f - UE_KINDA_SMALL_NUMBER))
			{
				Result += Eval(P0, P1, P2, P3, t);
			}
		}
	}

	return Result;
}

void ClosestPointApproximate(const FVector& FromPoint, const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, FVector& OutClosestPoint, float& OutClosestT, const int Steps)
{
	float ClosestDistanceSqr = TNumericLimits<float>::Max();
	FVector ClosestPoint = P0;
	float ClosestT = 0.0f;

	const int NumSteps = FMath::Max(1, Steps);
	const float Step = 1.0f / (float)NumSteps;

	FVector PrevPoint = P0;

	for (int j = 0; j < Steps; j++)
	{
		const float t = (float)(j + 1) * Step;
		const FVector CurrPoint = Eval(P0, P1, P2, P3, t);

		const FVector Closest = FMath::ClosestPointOnSegment(FromPoint, PrevPoint, CurrPoint);

		const float DistanceSqr = FVector::DistSquared(Closest, FromPoint);
		if (DistanceSqr < ClosestDistanceSqr)
		{
			ClosestDistanceSqr = DistanceSqr;
			ClosestPoint = Closest;
			// The SegT is calculate by the closest point calculation above, but it does not return in!
			const float SegLengthSqr = FVector::DistSquared(PrevPoint, CurrPoint);
			const float SegT = SegLengthSqr > UE_KINDA_SMALL_NUMBER ? FVector::DistSquared(PrevPoint, Closest) / SegLengthSqr : 0.0f;
			ClosestT = (t - Step) + SegT * Step;
		}

		PrevPoint = CurrPoint;
	}

	OutClosestPoint = ClosestPoint;
	OutClosestT = ClosestT;
}

void SegmentClosestPointApproximate(const FVector& SegStart, const FVector& SegEnd, const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, FVector& OutClosestPoint, float& OutClosestT, const int Steps)
{
	float ClosestDistanceSqr = TNumericLimits<float>::Max();
	FVector ClosestPoint = P0;
	float ClosestT = 0.0f;

	const int NumSteps = FMath::Max(1, Steps);
	const float Step = 1.0f / (float)NumSteps;

	FVector PrevPoint = P0;

	for (int j = 0; j < NumSteps; j++)
	{
		const float t = (float)(j + 1) * Step;
		const FVector CurrPoint = Eval(P0, P1, P2, P3, t);

		FVector Closest;
		FVector SegClosest;
		FMath::SegmentDistToSegmentSafe(PrevPoint, CurrPoint, SegStart, SegEnd, Closest, SegClosest);

		const float DistanceSqr = FVector::DistSquared(Closest, SegClosest);
		if (DistanceSqr < ClosestDistanceSqr)
		{
			ClosestDistanceSqr = DistanceSqr;
			ClosestPoint = Closest;
			// The SegT is calculate by the closest point calculation above, but it does not return in!
			const float SegLengthSqr = FVector::DistSquared(PrevPoint, CurrPoint);
			const float SegT = SegLengthSqr > UE_KINDA_SMALL_NUMBER ? FVector::DistSquared(PrevPoint, Closest) / SegLengthSqr : 0.0f;
			ClosestT = (t - Step) + SegT * Step;
		}

		PrevPoint = CurrPoint;
	}

	OutClosestPoint = ClosestPoint;
	OutClosestT = ClosestT;
}

float ArcLengthApproximate(const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3)
{
	FVector Result[5];
	SplitAt(P0, P1, P2, P3, 0.5f, Result);
	return FVector::Dist(P0, Result[0]) + FVector::Dist(Result[1], Result[2]) + FVector::Dist(Result[2], Result[3]) + FVector::Dist(Result[3], Result[4]) + FVector::Dist(Result[4], P3);
}

void TessellateRecursive(TArray<FVector>& Output, const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, const float ToleranceSqr, int Level, const int MaxLevel)
{
	if (Level >= MaxLevel)
	{
		Output.Add(P3);
		return;
	}

	// Handle degenerate segment.
	FVector Dir = P3 - P0;
	if (Dir.IsNearlyZero())
	{
		Output.Add(P3);
		return;
	}

	// If the control points are close enough to approximate a line within tolerance, stop recursing.
	Dir = Dir.GetUnsafeNormal();
	const FVector RelP1 = P1 - P0;
	const FVector RelP2 = P2 - P0;
	const FVector ProjP1 = Dir * FVector::DotProduct(Dir, RelP1);
	const FVector ProjP2 = Dir * FVector::DotProduct(Dir, RelP2);
	const float DistP1Sqr = FVector::DistSquared(RelP1, ProjP1);
	const float DistP2Sqr = FVector::DistSquared(RelP2, ProjP2);
	if (DistP1Sqr < ToleranceSqr && DistP2Sqr < ToleranceSqr)
	{
		Output.Add(P3);
		return;
	}

	// Split the curve in half and recurse.
	const FVector P01 = FMath::Lerp(P0, P1, 0.5f);
	const FVector P12 = FMath::Lerp(P1, P2, 0.5f);
	const FVector P23 = FMath::Lerp(P2, P3, 0.5f);
	const FVector P012 = FMath::Lerp(P01, P12, 0.5f);
	const FVector P123 = FMath::Lerp(P12, P23, 0.5f);
	const FVector P0123 = FMath::Lerp(P012, P123, 0.5f);

	TessellateRecursive(Output, P0, P01, P012, P0123, ToleranceSqr, Level + 1, MaxLevel);
	TessellateRecursive(Output, P0123, P123, P23, P3, ToleranceSqr, Level + 1, MaxLevel);
}

void Tessellate(TArray<FVector>& Output, const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, const float Tolerance, const int MaxLevel)
{
	TessellateRecursive(Output, P0, P1, P2, P3, Tolerance * Tolerance, 0, MaxLevel);
}

} // namespace CubicBezier
} // namespace UE
