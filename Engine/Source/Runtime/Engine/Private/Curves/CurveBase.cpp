// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/CurveBase.h"

#include "JsonObjectConverter.h"
#include "Serialization/Csv/CsvParser.h"
#include "EditorFramework/AssetImportData.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveBase)


/* UCurveBase interface
 *****************************************************************************/

UCurveBase::UCurveBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ }


void UCurveBase::GetTimeRange(float& MinTime, float& MaxTime) const
{
	TArray<FRichCurveEditInfoConst, TInlineAllocator<16>> Curves;
	GetCurves(Curves);

	if (Curves.Num() > 0)
	{
		check(Curves[0].CurveToEdit);
		Curves[0].CurveToEdit->GetTimeRange(MinTime, MaxTime);

		for (int32 i=1; i<Curves.Num(); i++)
		{
			float CurveMin, CurveMax;
			check(Curves[i].CurveToEdit != NULL);
			Curves[i].CurveToEdit->GetTimeRange(CurveMin, CurveMax);

			MinTime = FMath::Min(CurveMin, MinTime);
			MaxTime = FMath::Max(CurveMax, MaxTime);
		}
	}
}
void UCurveBase::GetTimeRange(double& MinTime, double& MaxTime) const
{
	float Min = MinTime, Max = MaxTime;
	GetTimeRange(Min, Max);
	MinTime = Min;
	MaxTime = Max;
}

void UCurveBase::GetValueRange(float& MinValue, float& MaxValue) const
{
	TArray<FRichCurveEditInfoConst, TInlineAllocator<16>> Curves;
	GetCurves(Curves);

	if (Curves.Num() > 0)
	{
		check(Curves[0].CurveToEdit);
		Curves[0].CurveToEdit->GetValueRange(MinValue, MaxValue);

		for (int32 i=1; i<Curves.Num(); i++)
		{
			float CurveMin, CurveMax;
			check(Curves[i].CurveToEdit != NULL);
			Curves[i].CurveToEdit->GetValueRange(CurveMin, CurveMax);

			MinValue = FMath::Min(CurveMin, MinValue);
			MaxValue = FMath::Max(CurveMax, MaxValue);
		}
	}
}
void UCurveBase::GetValueRange(double& MinValue, double& MaxValue) const
{
	float Min = MinValue, Max = MaxValue;
	GetValueRange(Min, Max);
	MinValue = Min;
	MaxValue = Max;
}

void UCurveBase::ModifyOwner() 
{
	Modify(true);
}

TArray<const UObject*> UCurveBase::GetOwners() const
{
	TArray<const UObject*> Owners;
	Owners.Add(this);		// CurveBase owns its own curve

	return Owners;
}


void UCurveBase::MakeTransactional() 
{
	SetFlags(GetFlags() | RF_Transactional);
}


void UCurveBase::SetOnCurveChangedIsInteractive(bool bInteractive)
{
#if WITH_EDITOR
	bOnCurveUpdateInteractive = bInteractive;
#endif
}

void UCurveBase::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
#if WITH_EDITOR
	// ideally OnCurveChanged should have taken a EPropertyChangeType as a function arg
	//	instead we pick it up from an extra call to SetOnCurveChangedIsInteractive
	EPropertyChangeType::Type ChangeType = bOnCurveUpdateInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet;
	OnUpdateCurve.Broadcast(this, ChangeType);
#endif // WITH_EDITOR
}


void UCurveBase::ResetCurve()
{
	TArray<FRichCurveEditInfo> Curves = GetCurves();

	for (int32 CurveIdx=0; CurveIdx<Curves.Num(); CurveIdx++)
	{
		if (Curves[CurveIdx].CurveToEdit != NULL)
		{
			Curves[CurveIdx].CurveToEdit->Reset();
		}
	}
}


TArray<FString> UCurveBase::CreateCurveFromCSVString(const FString& InString)
{
	// Array used to store problems about curve import
	TArray<FString> OutProblems;

	TArray<FRichCurveEditInfo> Curves = GetCurves();
	const int32 NumCurves = Curves.Num();

	const FCsvParser Parser(InString);
	const FCsvParser::FRows& Rows = Parser.GetRows();

	if (Rows.Num() == 0)
	{
		OutProblems.Add(FString("No data."));
		return OutProblems;
	}

	// First clear out old data.
	ResetCurve();

	// Each row represents a point
	for (int32 RowIdx=0; RowIdx<Rows.Num(); RowIdx++)
	{
		const TArray<const TCHAR*>& Cells = Rows[RowIdx];
		const int32 NumCells = Cells.Num();

		// Need at least two cell, Time and one Value
		if (NumCells < 2)
		{
			OutProblems.Add(FString::Printf(TEXT("Row '%d' has less than 2 cells."), RowIdx));
			continue;
		}

		float Time = FCString::Atof(Cells[0]);

		for (int32 CellIdx=1; CellIdx<NumCells && CellIdx<(NumCurves+1); CellIdx++)
		{
			FRealCurve* Curve = Curves[CellIdx-1].CurveToEdit;
			if (Curve != NULL)
			{
				FKeyHandle KeyHandle = Curve->AddKey(Time, FCString::Atof(Cells[CellIdx]));
				Curve->SetKeyInterpMode(KeyHandle, RCIM_Linear);
			}
		}

		if (NumCells > (NumCurves + 1))
		{
			// If we get more cells than curves (+1 for time cell)
			OutProblems.Add(FString::Printf(TEXT("Row '%d' has too many cells for the curve(s)."), RowIdx));
		}
		else if (NumCells < (NumCurves + 1))
		{
			// If we got too few cells
			OutProblems.Add(FString::Printf(TEXT("Row '%d' has too few cells for the curve(s)."), RowIdx));
		}
	}

	Modify(true);

	return OutProblems;
}

void UCurveBase::ImportFromJSONString(const FString& InString, TArray<FString>& OutProblems)
{
	TSharedPtr<FJsonObject> JsonObject;
	{
		const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(InString);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
		{
			OutProblems.Add(FString::Printf(TEXT("Failed to parse the JSON data. Error: %s"), *JsonReader->GetErrorMessage()));
			return;
		}
	}

	const TSharedRef<FJsonObject> BackupJSONObject = MakeShared<FJsonObject>();
	if (!FJsonObjectConverter::UStructToJsonAttributes(GetClass(), this, BackupJSONObject))
	{
		OutProblems.Add(FString::Printf(TEXT("Failed to backup existing data before import.")));
		return;
	}
	
	if (!FJsonObjectConverter::JsonAttributesToUStruct(JsonObject->Values, GetClass(), this, 0, CPF_Deprecated | CPF_Transient, true))
	{
		// Rollback any changes made during import with the backup
		FJsonObjectConverter::JsonAttributesToUStruct(BackupJSONObject->Values, GetClass(), this, 0, CPF_Deprecated | CPF_Transient, true);
		
		OutProblems.Add(FString::Printf(TEXT("Failed to import JSON data. Check logs for details.")));
		return;
	}

	Modify(true);
}

FString UCurveBase::ExportAsJSONString() const
{
	FString Result;
	const TSharedRef<FJsonObject> JSONObject = MakeShared<FJsonObject>();
	if (FJsonObjectConverter::UStructToJsonAttributes(GetClass(), this, JSONObject, 0, CPF_Deprecated | CPF_Transient))
	{
		const TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&Result);
		FJsonSerializer::Serialize(JSONObject, JsonWriter);
	}
	return Result;
}

/* UObject interface
 *****************************************************************************/

#if WITH_EDITORONLY_DATA
void UCurveBase::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}

	Super::GetAssetRegistryTags(Context);
}

void UCurveBase::PostInitProperties()
{
	if (IsAsset())
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}

	Super::PostInitProperties();
}

void UCurveBase::PostLoad()
{
	Super::PostLoad();

	if (!IsAsset() && AssetImportData)
	{
		// UCurves inside Blueprints previously created these sub objects incorrectly, so ones loaded off disk will still exist
		AssetImportData = nullptr;
	}

	if (!ImportPath_DEPRECATED.IsEmpty() && AssetImportData)
	{
		FAssetImportInfo Info;
		Info.Insert(FAssetImportInfo::FSourceFile(ImportPath_DEPRECATED));
		AssetImportData->SourceData = MoveTemp(Info);
	}
}
#endif //WITH_EDITORONLY_DATA

#if WITH_EDITOR

void UCurveBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnUpdateCurve.Broadcast(this, PropertyChangedEvent.ChangeType);
}

#endif // WITH_EDITOR
