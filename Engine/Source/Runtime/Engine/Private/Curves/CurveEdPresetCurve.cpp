// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CurveEdPresetCurve.cpp: Preset curve implementation.
=============================================================================*/

#include "Curves/CurveEdPresetCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEdPresetCurve)

/**
 *	UCurveEdPresetCurve
 */

UDEPRECATED_CurveEdPresetCurve::UDEPRECATED_CurveEdPresetCurve(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

