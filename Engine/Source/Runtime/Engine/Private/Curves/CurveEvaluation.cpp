// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/CurveEvaluation.h"
#include "Curves/RichCurve.h"

namespace UE
{
	namespace Curves
	{		
		int32 SolveCubic(double Coeff[4], double Solution[3])
		{
			auto Cbrt = [](double x) -> double
			{
				return ((x) > 0.0 ? pow((x), 1.0 / 3.0) : ((x) < 0.0 ? -pow((double)-(x), 1.0 / 3.0) : 0.0));
			};
			int32 NumSolutions = 0;

			// Normal form: x^3 + Ax^2 + Bx + C = 0
			double Denominator = (Coeff[3] != 0.0) ? Coeff[3] : UE_DOUBLE_SMALL_NUMBER * UE_DOUBLE_SMALL_NUMBER;
			double A = Coeff[2] / Denominator;
			double B = Coeff[1] / Denominator;
			double C = Coeff[0] / Denominator;

			// Substitute x = y - A/3 to eliminate quadric term: x^3 +px + q = 0
			double SqOfA = A * A;
			double P = 1.0 / 3 * (-1.0 / 3 * SqOfA + B);
			double Q = 1.0 / 2 * (2.0 / 27 * A * SqOfA - 1.0 / 3 * A * B + C);

			// Use Cardano's formula
			double CubeOfP = P * P * P;
			double D = Q * Q + CubeOfP;
			if (FMath::IsNearlyZero(D))
			{
				// One triple solution
				if (FMath::IsNearlyZero(Q))
				{
					Solution[0] = 0;
					NumSolutions = 1;
				}
				// One single and one double solution
				else
				{
					double U = Cbrt(-Q);
					Solution[0] = 2 * U;
					Solution[1] = -U;
					NumSolutions = 2;
				}
			}
			// Three real solutions
			else if (D < 0)
			{
				double Phi = 1.0 / 3 * acos(-Q / sqrt(-CubeOfP));
				double T = 2 * sqrt(-P);

				Solution[0] = T * cos(Phi);
				Solution[1] = -T * cos(Phi + UE_DOUBLE_PI / 3);
				Solution[2] = -T * cos(Phi - UE_DOUBLE_PI / 3);
				NumSolutions = 3;
			}
			// One real solution
			else
			{
				double SqrtD = sqrt(D);
				double U = Cbrt(SqrtD - Q);
				double V = -Cbrt(SqrtD + Q);

				Solution[0] = U + V;
				NumSolutions = 1;
			}

			// Resubstitute
			double Sub = 1.0 / 3 * A;
			for (int32 Index = 0; Index < NumSolutions; ++Index)
			{
				Solution[Index] -= Sub;
			}

			return NumSolutions;
		}

		void BezierToPower(double A1, double B1, double C1, double D1, double* A2, double* B2, double* C2, double* D2)
		{
			double A = B1 - A1;
			double B = C1 - B1;
			double C = D1 - C1;
			double D = B - A;
			*A2 = C - B - D;
			*B2 = 3.0 * D;
			*C2 = 3.0 * A;
			*D2 = A1;
		}

		float WeightedEvalForTwoKeys(
			float Key1Value, float Key1Time, float Key1LeaveTangent, float Key1LeaveTangentWeight, ERichCurveTangentWeightMode Key1TangentWeightMode,
			float Key2Value, float Key2Time, float Key2ArriveTangent, float Key2ArriveTangentWeight,  ERichCurveTangentWeightMode Key2TangentWeightMode,
			float InTime)
		{
			const float Diff = Key2Time - Key1Time;
			const float Alpha = (InTime - Key1Time) / Diff;
			const float OneThird = 1.0f / 3.0f;
			const double Time1 = Key1Time;
			const double Time2 = Key2Time;
			const float X = Time2 - Time1;
			float CosAngle, SinAngle;
			float Angle = FMath::Atan(Key1LeaveTangent);
			FMath::SinCos(&SinAngle, &CosAngle, Angle);
			float LeaveWeight;
			if (Key1TangentWeightMode == RCTWM_WeightedNone || Key1TangentWeightMode == RCTWM_WeightedArrive)
			{
				const float LeaveTangentNormalized = Key1LeaveTangent;
				const float Y = LeaveTangentNormalized * X;
				LeaveWeight = FMath::Sqrt(X * X + Y * Y) * OneThird;
			}
			else
			{
				LeaveWeight = Key1LeaveTangentWeight;
			}
			const float Key1TanX = CosAngle * LeaveWeight + Time1;
			const float Key1TanY = SinAngle * LeaveWeight + Key1Value;

			Angle = FMath::Atan(Key2ArriveTangent);
			FMath::SinCos(&SinAngle, &CosAngle, Angle);
			float ArriveWeight;
			if (Key2TangentWeightMode == RCTWM_WeightedNone || Key2TangentWeightMode == RCTWM_WeightedLeave)
			{
				const float ArriveTangentNormalized = Key2ArriveTangent;
				const float Y = ArriveTangentNormalized * X;
				ArriveWeight = FMath::Sqrt(X * X + Y * Y) * OneThird;
			}
			else
			{
				ArriveWeight = Key2ArriveTangentWeight;
			}
			const float Key2TanX = -CosAngle * ArriveWeight + Time2;
			const float Key2TanY = -SinAngle * ArriveWeight + Key2Value;

			//Normalize the Time Range
			const float RangeX = Time2 - Time1;

			const float Dx1 = Key1TanX - Time1;
			const float Dx2 = Key2TanX - Time1;

			// Normalize values
			const float NormalizedX1 = Dx1 / RangeX;
			const float NormalizedX2 = Dx2 / RangeX;

			double Coeff[4];
			double Results[3];

			//Convert Bezier to Power basis, also float to double for precision for root finding.
			BezierToPower(
				0.0, NormalizedX1, NormalizedX2, 1.0,
				&(Coeff[3]), &(Coeff[2]), &(Coeff[1]), &(Coeff[0])
			);

			Coeff[0] = Coeff[0] - Alpha;

			const int32 NumResults = SolveCubic(Coeff, Results);
			double NewInterp = Alpha;
			if (NumResults == 1)
			{
				NewInterp = Results[0];
			}
			else
			{
				NewInterp = TNumericLimits<double>::Lowest(); //just need to be out of range
				for (double Result : Results)
				{
					if ((Result > 0.0 || FMath::IsNearlyEqual(Result, 0.0)) && (Result < 1.0 || FMath::IsNearlyEqual(Result, 1.0))) 
					{
						if (NewInterp < 0.0 || Result > NewInterp)
						{
							NewInterp = Result;
						}
					}
				}

				if (NewInterp == TNumericLimits<double>::Lowest())
				{
					NewInterp = 0.0;
				}

			}
			//now use NewInterp and adjusted tangents plugged into the Y (Value) part of the graph.
			const float P0 = Key1Value;
			const float P1 = Key1TanY;
			const float P3 = Key2Value;
			const float P2 = Key2TanY;

			return BezierInterp(P0, P1, P2, P3, NewInterp);
		}

		bool IsItNotWeighted(const FRichCurveKey& Key1, const FRichCurveKey& Key2)
		{
			return ((Key1.TangentWeightMode == RCTWM_WeightedNone || Key1.TangentWeightMode == RCTWM_WeightedArrive)
				&& (Key2.TangentWeightMode == RCTWM_WeightedNone || Key2.TangentWeightMode == RCTWM_WeightedLeave));
		}

		bool IsWeighted(const FRichCurveKey& Key1, const FRichCurveKey& Key2)
		{
			return (Key1.TangentMode != RCTM_None && (Key1.TangentWeightMode == RCTWM_WeightedBoth || Key1.TangentWeightMode == RCTWM_WeightedLeave)) &&
				(Key2.TangentMode != RCTM_None && (Key2.TangentWeightMode == RCTWM_WeightedBoth || Key2.TangentWeightMode == RCTWM_WeightedArrive));
		}

		float EvalForTwoKeys(const FRichCurveKey& Key1, const FRichCurveKey& Key2, const float InTime)
		{
			const float Diff = Key2.Time - Key1.Time;

			if (Diff > 0.f)
			{
				if (Key1.InterpMode != RCIM_Constant)
				{
					const float Alpha = (InTime - Key1.Time) / Diff;
					const float P0 = Key1.Value;
					const float P3 = Key2.Value;

					if (Key1.InterpMode == RCIM_Linear)
					{
						return FMath::Lerp(P0, P3, Alpha);
					}
					else 
					{					
						if (IsItNotWeighted(Key1,Key2))
						{
							const float OneThird = 1.0f / 3.0f;
							const float P1 = P0 + (Key1.LeaveTangent * Diff * OneThird);
							const float P2 = P3 - (Key2.ArriveTangent * Diff * OneThird);

							return BezierInterp(P0, P1, P2, P3, Alpha);
						}
						//if (IsWeighted(Key1, Key2))//it's weighted
						else	
						{
						return WeightedEvalForTwoKeys(
							Key1.Value, Key1.Time, Key1.LeaveTangent, Key1.LeaveTangentWeight, Key1.TangentWeightMode,
							Key2.Value, Key2.Time, Key2.ArriveTangent, Key2.ArriveTangentWeight, Key2.TangentWeightMode,
							InTime);
						}
					}
				}
				else
				{
					return InTime < Key2.Time ? Key1.Value : Key2.Value;
				}
			}
			else
			{
				return Key1.Value;
			}
		}
	}
}
