// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CurveFloat.cpp
=============================================================================*/

#include "Curves/CurveFloat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveFloat)

FRuntimeFloatCurve::FRuntimeFloatCurve()
	: ExternalCurve(nullptr)
{
}

FRichCurve* FRuntimeFloatCurve::GetRichCurve()
{
	if (ExternalCurve != nullptr)
	{
		return &(ExternalCurve->FloatCurve);
	}
	else
	{
		return &EditorCurveData;
	}
}

const FRichCurve* FRuntimeFloatCurve::GetRichCurveConst() const
{
	if (ExternalCurve != nullptr)
	{
		return &(ExternalCurve->FloatCurve);
	}
	else
	{
		return &EditorCurveData;
	}
}

UCurveFloat::UCurveFloat(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

float UCurveFloat::GetFloatValue( float InTime ) const
{
	return FloatCurve.Eval(InTime);
}

TArray<FRichCurveEditInfoConst> UCurveFloat::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(FRichCurveEditInfoConst(&FloatCurve));
	return Curves;
}

void UCurveFloat::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	Curves.Add(FRichCurveEditInfoConst(&FloatCurve));
}

TArray<FRichCurveEditInfo> UCurveFloat::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(FRichCurveEditInfo(&FloatCurve, FName(*GetName())));
	return Curves;
}

bool UCurveFloat::IsValidCurve( FRichCurveEditInfo CurveInfo )
{
	return CurveInfo.CurveToEdit == &FloatCurve;
}

bool UCurveFloat::operator==( const UCurveFloat& Curve ) const
{
	return bIsEventCurve == Curve.bIsEventCurve && FloatCurve == Curve.FloatCurve;
}

