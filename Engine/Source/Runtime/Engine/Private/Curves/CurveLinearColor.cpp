// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UCurveLinearColor.cpp
=============================================================================*/

#include "Curves/CurveLinearColor.h"
#include "CanvasItem.h"
#include "Math/Float16Color.h"
#include "UObject/ReleaseObjectVersion.h"
#include "Logging/MessageLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveLinearColor)

FLinearColor FRuntimeCurveLinearColor::GetLinearColorValue(float InTime) const
{
	if (ExternalCurve)
	{
		return ExternalCurve->GetLinearColorValue(InTime);
	}

	FLinearColor Result;
	Result.R = ColorCurves[0].Eval(InTime);
	Result.G = ColorCurves[1].Eval(InTime);
	Result.B = ColorCurves[2].Eval(InTime);

	// No alpha keys means alpha should be 1
	if (ColorCurves[3].GetNumKeys() == 0)
	{
		Result.A = 1.0f;
	}
	else
	{
		Result.A = ColorCurves[3].Eval(InTime);
	}

	return Result;
}

UCurveLinearColor::UCurveLinearColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AdjustHue(0.0f)
	, AdjustSaturation(1.0f)
	, AdjustBrightness(1.0f)
	, AdjustBrightnessCurve(1.0f)
	, AdjustVibrance(0.0f)
	, AdjustMinAlpha(0.0f)
	, AdjustMaxAlpha(1.0f)
{
#if WITH_EDITOR
	// if the curve is empty
	if (FloatCurves[0].GetNumKeys() == 0
		&& FloatCurves[1].GetNumKeys() == 0
		&& FloatCurves[2].GetNumKeys() == 0)
	{
		// Add a key for Black at 0...
		FloatCurves[0].AddKey(0.f, 0.f);
		FloatCurves[1].AddKey(0.f, 0.f);
		FloatCurves[2].AddKey(0.f, 0.f);
		FloatCurves[3].AddKey(0.f, 1.f);

		//...and a key for White at 1
		FloatCurves[0].AddKey(1.f, 1.f);
		FloatCurves[1].AddKey(1.f, 1.f);
		FloatCurves[2].AddKey(1.f, 1.f);
		FloatCurves[3].AddKey(1.f, 1.f);
	}
#endif
}

FLinearColor UCurveLinearColor::GetLinearColorValue( float InTime ) const
{
	return GetAdjustedColorValue(InTime, false);
}

FLinearColor UCurveLinearColor::GetClampedLinearColorValue(float InTime) const
{
	return GetAdjustedColorValue(InTime, true);
}

FLinearColor UCurveLinearColor::GetAdjustedColorValue(float InTime, bool bClampOver1Always) const
{
	// Logic copied from .\Engine\Source\Developer\TextureCompressor\Private\TextureCompressorModule.cpp , from AdjustColorsOld
	//  @todo : see fixes in AdjustColorsNew
	const FLinearColor OriginalColor = GetUnadjustedLinearColorValue(InTime);
	
	// note RGB can be negative or > 1.0 here

	// if bClampOver1Always , always clamp to stay in[ 0,1]
	// else, Only clamp value if the color is input RGB <= 1 (fuzzy)
	const bool bShouldClampValue = bClampOver1Always || 
		( FMath::Max3(OriginalColor.R,OriginalColor.G,OriginalColor.B) < 1.0f+UE_KINDA_SMALL_NUMBER );

	// Convert to HSV (note RGB can be out of [0,1] range expected by HSV here)
	FLinearColor HSVColor = OriginalColor.LinearRGBToHSV();
	float& PixelHue = HSVColor.R;
	float& PixelSaturation = HSVColor.G;
	float& PixelValue = HSVColor.B;

	// Apply brightness adjustment
	PixelValue *= AdjustBrightness;

	// Apply brightness power adjustment
	if (!FMath::IsNearlyEqual(AdjustBrightnessCurve, 1.0f, (float)UE_KINDA_SMALL_NUMBER) && AdjustBrightnessCurve != 0.0f)
	{
		// Raise HSV.V to the specified power
		PixelValue = FMath::Pow(PixelValue, AdjustBrightnessCurve);
	}

	// Apply "vibrancy" adjustment
	if (!FMath::IsNearlyZero(AdjustVibrance, (float)UE_KINDA_SMALL_NUMBER))
	{
		const float SatRaisePow = 5.0f;
		const float InvSatRaised = FMath::Pow(1.0f - PixelSaturation, SatRaisePow);

		const float ClampedVibrance = FMath::Clamp(AdjustVibrance, 0.0f, 1.0f);
		const float HalfVibrance = ClampedVibrance * 0.5f;

		const float SatProduct = HalfVibrance * InvSatRaised;

		PixelSaturation += SatProduct;
	}

	// Apply saturation adjustment
	PixelSaturation *= AdjustSaturation;

	// Apply hue adjustment
	PixelHue += AdjustHue;

	// Clamp HSV values
	// HSVToLinearRGB now does the needed clamps, except for Value
	if (bShouldClampValue)
	{
		PixelValue = FMath::Clamp(PixelValue, 0.0f, 1.0f);
	}

	// Convert back to a linear color
	FLinearColor LinearColor = HSVColor.HSVToLinearRGB();

	// Remap the alpha channel
	LinearColor.A = FMath::Lerp(AdjustMinAlpha, AdjustMaxAlpha, OriginalColor.A);
	return LinearColor;
}

FLinearColor UCurveLinearColor::GetUnadjustedLinearColorValue(float InTime) const
{
	FLinearColor Result;

	Result.R = FloatCurves[0].Eval(InTime);
	Result.G = FloatCurves[1].Eval(InTime);
	Result.B = FloatCurves[2].Eval(InTime);

	// No alpha keys means alpha should be 1
	if (FloatCurves[3].GetNumKeys() == 0)
	{
		Result.A = 1.0f;
	}
	else
	{
		Result.A = FloatCurves[3].Eval(InTime);
	}
	
	// Result outside [0,1] intentionally allowed here

	return Result;
}

static const FName RedCurveName("R");
static const FName GreenCurveName("G");
static const FName BlueCurveName("B");
static const FName AlphaCurveName("A");

TArray<FRichCurveEditInfoConst> UCurveLinearColor::GetCurves() const 
{
	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[0], RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[1], GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[2], BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[3], AlphaCurveName));
	return Curves;
}

void UCurveLinearColor::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const 
{
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[0], RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[1], GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[2], BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[3], AlphaCurveName));
}

TArray<FRichCurveEditInfo> UCurveLinearColor::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(FRichCurveEditInfo(&FloatCurves[0], RedCurveName));
	Curves.Add(FRichCurveEditInfo(&FloatCurves[1], GreenCurveName));
	Curves.Add(FRichCurveEditInfo(&FloatCurves[2], BlueCurveName));
	Curves.Add(FRichCurveEditInfo(&FloatCurves[3], AlphaCurveName));
	return Curves;
}

bool UCurveLinearColor::operator==( const UCurveLinearColor& Curve ) const
{
	return (FloatCurves[0] == Curve.FloatCurves[0]) && (FloatCurves[1] == Curve.FloatCurves[1]) && (FloatCurves[2] == Curve.FloatCurves[2]) && (FloatCurves[3] == Curve.FloatCurves[3]) ;
}

bool UCurveLinearColor::IsValidCurve( FRichCurveEditInfo CurveInfo )
{
	return CurveInfo.CurveToEdit == &FloatCurves[0] ||
		CurveInfo.CurveToEdit == &FloatCurves[1] ||
		CurveInfo.CurveToEdit == &FloatCurves[2] ||
		CurveInfo.CurveToEdit == &FloatCurves[3];
}

#if WITH_EDITOR

void UCurveLinearColor::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UCurveLinearColor::DrawThumbnail(FCanvas* Canvas, FVector2D StartXY, FVector2D SizeXY)
{
	// this draws the curve into the thumbnail texture
	//	typically start == {0,0} and size == {256,256}
	// this updates continuously when your mouse hovers over the thumbnail
	FCanvasTileItem DrawItem(StartXY, FVector2D(1.f, SizeXY.Y), FLinearColor::Black);

	// Render the gradient
	float Width = SizeXY.X;
	check( Width > 1.f );
	for (float x = 0.f; x < Width; ++x)
	{
		DrawItem.Position.X = StartXY.X + x;

		FLinearColor Color = GetLinearColorValue(x * (1.f / (Width-1)) );

		DrawItem.SetColor(Color);
		DrawItem.Draw(Canvas);
	}
}


void UCurveLinearColor::PushToSourceData(TArrayView<FFloat16Color> &SrcData, int32 Start, int32 Width)
{
	check( Width > 1 );
	for (int32 x = 0; x < Width; x++)
	{
		SrcData[Start + x] = GetLinearColorValue(x * (1.f / (Width-1)) );
	}
}

void UCurveLinearColor::PushUnadjustedToSourceData(TArrayView<FFloat16Color>& SrcData, int32 Start, int32 Width)
{
	check( Width > 1 );
	for (int32 x = 0; x < Width; x++)
	{
		SrcData[Start + x] = GetUnadjustedLinearColorValue(x * (1.f / (Width-1)) );
	}
}

void UCurveLinearColor::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	// this triggers OnUpdateCurve delegate, which goes to UCurveLinearColorAtlas::OnCurveUpdated to redraw the atlas texture 
	Super::OnCurveChanged(ChangedCurveEditInfos);
}
#endif

void UCurveLinearColor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::UnclampRGBColorCurves)
	{
		FMessageLog("LoadErrors").Warning(FText::Format(NSLOCTEXT("CurveEditor", "CurveDataUpdate", "Linear color curves now accurately handle RGB values > 1. If you were relying on HSV clamping, please update {0}"),
			FText::FromString(GetName())));
	}
#endif
}

void UCurveLinearColor::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Super::Serialize(Ar);
}
