// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UCurveLinearColorAtlas.cpp
=============================================================================*/

#include "Curves/CurveLinearColorAtlas.h"
#include "Async/TaskGraphInterfaces.h"
#include "Curves/CurveLinearColor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveLinearColorAtlas)


UCurveLinearColorAtlas::UCurveLinearColorAtlas(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// a better default value for TextureSize is 32 or 64
	//	but that's not simple to change because of the way UProps serialize deltas
	//	instead we change the value in UCurveLinearColorAtlasFactory
	TextureSize = 256;

#if WITH_EDITORONLY_DATA
	MipGenSettings = TMGS_NoMipmaps;
#endif
	Filter = TextureFilter::TF_Bilinear;
	SRGB = false;
	AddressX = TA_Clamp;
	AddressY = TA_Clamp;
	CompressionSettings = TC_HDR;
#if WITH_EDITORONLY_DATA
	bDisableAllAdjustments = false;
	bHasCachedColorAdjustments = false;
#endif
}
#if WITH_EDITOR
bool UCurveLinearColorAtlas::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (bDisableAllAdjustments &&
		(InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustBrightness) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustBrightnessCurve) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustBrightness) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustSaturation) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustVibrance) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustRGBCurve) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustHue) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustMinAlpha) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, AdjustMaxAlpha) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, bChromaKeyTexture) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, ChromaKeyThreshold) ||
		 InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTexture, ChromaKeyColor)))
	{
		return false;
	}

	return true;
}

void UCurveLinearColorAtlas::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	// Determine whether any property that requires recompression of the texture, or notification to Materials has changed.
	bool bRequiresNotifyMaterials = false;

	if (PropertyChangedEvent.Property != nullptr)
	{
		const FName PropertyName(PropertyChangedEvent.Property->GetFName());
		// if Resizing
		if ( PropertyName == GET_MEMBER_NAME_CHECKED(UCurveLinearColorAtlas, TextureSize) )
		{
			TextureSize = FMath::Max<uint32>(TextureSize,2);

			UpdateTextures();
			bRequiresNotifyMaterials = true;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCurveLinearColorAtlas, GradientCurves))
		{
			// note: any old curves that were removed do not get their OnUpdateCurve delegate removed! (it still will notify me)
			//	but when we get a notification from them, we will then remove their OnUpdateCurve delegate

			for (int32 i = 0; i < GradientCurves.Num(); ++i)
			{
				if (GradientCurves[i] != nullptr)
				{
					// AddDelegateInstance will just keep adding more copies of the same delegate to the callback list
					//	first remove all notifications to me, then add :
					GradientCurves[i]->OnUpdateCurve.RemoveAll(this);
					GradientCurves[i]->OnUpdateCurve.AddUObject(this, &UCurveLinearColorAtlas::OnCurveUpdated);
				}
			}
			UpdateTextures();
			bRequiresNotifyMaterials = true;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCurveLinearColorAtlas, bDisableAllAdjustments))
		{
			if (bDisableAllAdjustments)
			{
				CacheAndResetColorAdjustments();
			}
			else
			{
				RestoreCachedColorAdjustments();
			}

			UpdateTextures();
			bRequiresNotifyMaterials = true;
		}
		else if (bDisableAllAdjustments)
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustBrightness))
			{
				AdjustBrightness = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustBrightnessCurve))
			{
				AdjustBrightnessCurve = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustBrightness))
			{
				AdjustBrightness = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustSaturation))
			{
				AdjustSaturation = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustVibrance))
			{
				AdjustVibrance = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustRGBCurve))
			{
				AdjustRGBCurve = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustHue))
			{
				AdjustHue = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustMinAlpha))
			{
				AdjustMinAlpha = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, AdjustMaxAlpha))
			{
				AdjustMaxAlpha = 0.0f;
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture, bChromaKeyTexture))
			{
				bChromaKeyTexture = false;
			}
		}
	}

	// Notify any loaded material instances if changed our compression format
	if (bRequiresNotifyMaterials)
	{
		NotifyMaterials();
	}
}

void UCurveLinearColorAtlas::CacheAndResetColorAdjustments()
{
	Modify();

	bHasCachedColorAdjustments = true;

	CachedColorAdjustments.bChromaKeyTexture = bChromaKeyTexture;
	CachedColorAdjustments.AdjustBrightness = AdjustBrightness;
	CachedColorAdjustments.AdjustBrightnessCurve = AdjustBrightnessCurve;
	CachedColorAdjustments.AdjustVibrance = AdjustVibrance;
	CachedColorAdjustments.AdjustSaturation = AdjustSaturation;
	CachedColorAdjustments.AdjustRGBCurve = AdjustRGBCurve;
	CachedColorAdjustments.AdjustHue = AdjustHue;
	CachedColorAdjustments.AdjustMinAlpha = AdjustMinAlpha;
	CachedColorAdjustments.AdjustMaxAlpha = AdjustMaxAlpha;

	AdjustBrightness = 1.0f;
	AdjustBrightnessCurve = 1.0f;
	AdjustVibrance = 0.0f;
	AdjustSaturation = 1.0f;
	AdjustRGBCurve = 1.0f;
	AdjustHue = 0.0f;
	AdjustMinAlpha = 0.0f;
	AdjustMaxAlpha = 1.0f;
	bChromaKeyTexture = false;
}

void UCurveLinearColorAtlas::RestoreCachedColorAdjustments()
{
	if (bHasCachedColorAdjustments)
	{
		Modify();

		AdjustBrightness = CachedColorAdjustments.AdjustBrightness;
		AdjustBrightnessCurve = CachedColorAdjustments.AdjustBrightnessCurve;
		AdjustVibrance = CachedColorAdjustments.AdjustVibrance;
		AdjustSaturation = CachedColorAdjustments.AdjustSaturation;
		AdjustRGBCurve = CachedColorAdjustments.AdjustRGBCurve;
		AdjustHue = CachedColorAdjustments.AdjustHue;
		AdjustMinAlpha = CachedColorAdjustments.AdjustMinAlpha;
		AdjustMaxAlpha = CachedColorAdjustments.AdjustMaxAlpha;
		bChromaKeyTexture = CachedColorAdjustments.bChromaKeyTexture;
	}
}
#endif

void UCurveLinearColorAtlas::PostLoad()
{
#if WITH_EDITOR
	for (int32 i = 0; i < GradientCurves.Num(); ++i)
	{
		if (GradientCurves[i] != nullptr)
		{
			GradientCurves[i]->OnUpdateCurve.AddUObject(this, &UCurveLinearColorAtlas::OnCurveUpdated);
		}
	}
	
	// re-draw into the Texture Source on load
	//	when the code is stable this should be an unnecessary nop
	//	but it lets us refresh the data when the code changes
	if ( 1 )
	{
		UpdateTextures();
	}
#endif // #if WITH_EDITOR

	// Super is UTexture2D
	//	will do CachePlatformData for us

	Super::PostLoad();
}

#if WITH_EDITOR
static void RenderGradient(TArrayView<FFloat16Color>& InSrcData, UObject* Gradient, int32 Start, int32 Width, bool bUseUnadjustedColor)
{
	UCurveLinearColor* GradientCurve = Cast<UCurveLinearColor>(Gradient);
	if ( GradientCurve )
	{
		// Render a gradient
		if (bUseUnadjustedColor)
		{
			GradientCurve->PushUnadjustedToSourceData(InSrcData, Start, Width);
		}
		else
		{
			GradientCurve->PushToSourceData(InSrcData, Start, Width);
		}
	}
	else
	{
		FFloat16Color White16 = FLinearColor::White;

		for (int32 x = 0; x < Width; x++)
		{
			InSrcData[Start + x] = White16;
		}
	}
}

// Immediately render a new material to the specified slot index (SlotIndex must be within this section's range)
void UCurveLinearColorAtlas::OnCurveUpdated(UCurveBase* Curve, EPropertyChangeType::Type ChangeType)
{
	// @@ todo : this ("Interactive" branch) is broken (for dragging the handles at the top, it works for dragging points in the curve graph)
	//	in theory we should see "Interactive" ChangeType during curve drags
	//	but in fact UCurveBase::OnCurveChanged just always passes "ValueSet" even during mouse drags
	//	so we never see Interactive, and are always re-rendering the Atlas
	if (ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}

	/*
	UE_LOGF(LogCore,Display, "UCurveLinearColorAtlas::OnCurveUpdated : [%ls] from [%ls]", 
					*GetName(),*Curve->GetName()
					);	
	*/

	UCurveLinearColor* Gradient = CastChecked<UCurveLinearColor>(Curve);

	int32 SlotIndex = GradientCurves.Find(Gradient);

	if ( SlotIndex == INDEX_NONE )
	{
		// this Curve is no longer in my list
		//	eg. it was removed from my array
		//	do not notify me any more:
		Gradient->OnUpdateCurve.RemoveAll(this);
		return;
	}

	// @todo : for efficiency, could update just the one gradient instead of all?

	UpdateTextures();
}

// Render any textures
void UCurveLinearColorAtlas::UpdateTextures()
{
	LLM_SCOPE(ELLMTag::Textures);
	
	TextureSize = FMath::Max<uint32>(TextureSize,2);
	check( TextureSize > 1 );
	
	// TextureHeight set automatically from NumCurves
	uint32 NumCurves = GradientCurves.Num();
	uint32 TextureHeight = FMath::Max<uint32>(NumCurves,1);

	PreEditChange(nullptr);

	if (Source.GetSizeX() != TextureSize ||
		Source.GetSizeY() != TextureHeight ||
		Source.GetFormat() != TSF_RGBA16F )
	{
		Source.Init(TextureSize, TextureHeight, 1, 1, TSF_RGBA16F);
	}

	const int32 TextureNumPixels = TextureSize * TextureHeight;
	FFloat16Color * SrcData = (FFloat16Color *) Source.LockMip(0);

	TArrayView<FFloat16Color> SrcDataArrayView(SrcData,TextureNumPixels);
	check( Source.CalcMipSize(0) == SrcDataArrayView.NumBytes() );

	for (uint32 i = 0; i < NumCurves; ++i)
	{
		RenderGradient(SrcDataArrayView, GradientCurves[i], i*TextureSize, TextureSize, bDisableAllAdjustments);
	}

	FFloat16Color White16 = FLinearColor::White;

	for (uint32 y = NumCurves; y < TextureHeight; y++)
	{
		// this is only used when NumCurves == 1 , then TextureHeight == 1
		for (uint32 x = 0; x < TextureSize; x++)
		{
			SrcData[y*TextureSize + x] = White16;
		}
	}

	Source.UnlockMip(0);
	PostEditChange();
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
FGuid UCurveLinearColorAtlas::BuildLightingGuid()
{
	return BuildLightingGuidFromHash();
}
#endif // WITH_EDITORONLY_DATA

bool UCurveLinearColorAtlas::GetCurveIndex(UCurveLinearColor* InCurve, int32& Index)
{
	Index = GradientCurves.Find(InCurve);
	if (Index != INDEX_NONE)
	{
		return true;
	}
	return false;
}

bool UCurveLinearColorAtlas::GetCurvePosition(UCurveLinearColor* InCurve, float& Position)
{
	int32 Index = GradientCurves.Find(InCurve);
	Position = 0.0f;
	if (Index != INDEX_NONE)
	{
		Position = (float)Index;
		return true;
	}
	return false;
}

