// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/CurveOwnerInterface.h"


/* FCurveOwnerInterface
 *****************************************************************************/

FLinearColor FCurveOwnerInterface::GetCurveColor(FRichCurveEditInfo CurveInfo) const
{
	FString CurveName = CurveInfo.CurveName.ToString();

	if (CurveName == TEXT("X") || CurveName == TEXT("R"))
	{
		return FLinearColor(1.0f, 0.05f, 0.05f);
	}
	
	if (CurveName == TEXT("Y") || CurveName == TEXT("G"))
	{
		return FLinearColor(0.05f, 1.0f, 0.05f);
	}
	
	if (CurveName == TEXT("Z") || CurveName == TEXT("B"))
	{
		return FLinearColor(0.1f, 0.2f, 1.0f);
	}

	return FLinearColor(0.2f, 0.2f, 0.2f);
}
