// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CurveVector.cpp
=============================================================================*/

#include "Curves/CurveVector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveVector)

FVector FRuntimeVectorCurve::GetValue(float InTime) const
{
	if (ExternalCurve)
	{
		return ExternalCurve->GetVectorValue(InTime);
	}

	FVector Result;
	Result.X = VectorCurves[0].Eval(InTime);
	Result.Y = VectorCurves[1].Eval(InTime);
	Result.Z = VectorCurves[2].Eval(InTime);
	return Result;
}

FRichCurve* FRuntimeVectorCurve::GetRichCurve(int32 Index)
{
	if (Index < 0 || Index >= 3)
	{
		return nullptr; 
	}

	if (ExternalCurve != nullptr)
	{
		return &(ExternalCurve->FloatCurves[Index]);
	}
	else
	{
		return &(VectorCurves[Index]);
	}
}

const FRichCurve* FRuntimeVectorCurve::GetRichCurveConst(int32 Index) const
{
	if (Index < 0 || Index >= 3)
	{
		return nullptr; 
	}
	
	if (ExternalCurve != nullptr)
	{
		return &(ExternalCurve->FloatCurves[Index]);
	}
	else
	{
		return &(VectorCurves[Index]);
	}
}

UCurveVector::UCurveVector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FVector UCurveVector::GetVectorValue( float InTime ) const
{
	FVector Result;
	Result.X = FloatCurves[0].Eval(InTime);
	Result.Y = FloatCurves[1].Eval(InTime);
	Result.Z = FloatCurves[2].Eval(InTime);
	return Result;
}

static const FName XCurveName("X");
static const FName YCurveName("Y");
static const FName ZCurveName("Z");

TArray<FRichCurveEditInfoConst> UCurveVector::GetCurves() const 
{
	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[0], XCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[1], YCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[2], ZCurveName));
	return Curves;
}

void UCurveVector::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const 
{
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[0], XCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[1], YCurveName));
	Curves.Add(FRichCurveEditInfoConst(&FloatCurves[2], ZCurveName));
}

TArray<FRichCurveEditInfo> UCurveVector::GetCurves() 
{
	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(FRichCurveEditInfo(&FloatCurves[0], XCurveName));
	Curves.Add(FRichCurveEditInfo(&FloatCurves[1], YCurveName));
	Curves.Add(FRichCurveEditInfo(&FloatCurves[2], ZCurveName));
	return Curves;
}

bool UCurveVector::operator==( const UCurveVector& Curve ) const
{
	return (FloatCurves[0] == Curve.FloatCurves[0]) && (FloatCurves[1] == Curve.FloatCurves[1]) && (FloatCurves[2] == Curve.FloatCurves[2]) ;
}

bool UCurveVector::IsValidCurve( FRichCurveEditInfo CurveInfo )
{
	return CurveInfo.CurveToEdit == &FloatCurves[0] ||
		CurveInfo.CurveToEdit == &FloatCurves[1] ||
		CurveInfo.CurveToEdit == &FloatCurves[2];
}


