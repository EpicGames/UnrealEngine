// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/IndexedCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IndexedCurve)


/* FIndexedCurve interface
 *****************************************************************************/

int32 FIndexedCurve::GetIndexSafe(FKeyHandle KeyHandle) const
{
	return IsKeyHandleValid(KeyHandle) ? *KeyHandlesToIndices.Find(KeyHandle) : INDEX_NONE;
}

bool FIndexedCurve::IsKeyHandleValid(FKeyHandle KeyHandle) const
{
	const int32 NumKeys = GetNumKeys();
	EnsureAllIndicesHaveHandles_Internal(NumKeys);

	if (const int32* IndexPtr = FindIndex(KeyHandle))
	{
		return (*IndexPtr >= 0 && *IndexPtr < NumKeys);
	}
	return false;
}


/* FIndexedCurve implementation
 *****************************************************************************/

void FIndexedCurve::EnsureAllIndicesHaveHandles_Internal(const int32 NumKeys) const
{
	if (KeyHandlesToIndices.Num() != NumKeys)
	{
		KeyHandlesToIndices.EnsureAllIndicesHaveHandles(NumKeys);
	}
}

FKeyHandle FIndexedCurve::GetNextKey(FKeyHandle KeyHandle) const
{
	if (const int32* KeyIndex = FindIndex(KeyHandle))
	{
		const int32 NextKeyIndex = *KeyIndex + 1;

		if (NextKeyIndex < GetNumKeys())
		{
			return GetKeyHandle(NextKeyIndex);
		}
	}

	return FKeyHandle::Invalid();
}

FKeyHandle FIndexedCurve::GetPreviousKey(FKeyHandle KeyHandle) const
{
	if (const int32* KeyIndex = FindIndex(KeyHandle))
	{
		const int32 PrevKeyIndex = *KeyIndex - 1;

		if (PrevKeyIndex >= 0)
		{
			return GetKeyHandle(PrevKeyIndex);
		}
	}

	return FKeyHandle::Invalid();
}

int32 FIndexedCurve::GetIndex(FKeyHandle KeyHandle) const
{
	const int32* KeyIndex = FindIndex(KeyHandle);
	check(KeyIndex);
	return *KeyIndex;
}


FKeyHandle FIndexedCurve::GetKeyHandle(int32 KeyIndex) const
{
	if (KeyIndex >= 0 && KeyIndex < GetNumKeys())
	{
		EnsureIndexHasAHandle(KeyIndex);
		return *KeyHandlesToIndices.FindKey(KeyIndex);
	}

	return FKeyHandle::Invalid();
}

void FIndexedCurve::ShiftCurve(float DeltaTime)
{
	EnsureAllIndicesHaveHandles();

	TSet<FKeyHandle> KeyHandles;
	KeyHandles.Reserve(KeyHandlesToIndices.Num());

	for (auto It = KeyHandlesToIndices.CreateConstIterator(); It; ++It)
	{
		KeyHandles.Add(*It);
	}

	ShiftCurve(DeltaTime, KeyHandles);
}

void FIndexedCurve::ShiftCurve(float DeltaTime, const TSet<FKeyHandle>& KeyHandles)
{
	if (KeyHandles.Num() != 0)
	{
		for (auto It = KeyHandles.CreateConstIterator(); It; ++It)
		{
			const FKeyHandle& KeyHandle = *It;
			if (KeyHandlesToIndices.Find(KeyHandle) != nullptr)
			{
				SetKeyTime(KeyHandle, GetKeyTime(KeyHandle) + DeltaTime);
			}
		}
	}
}

void FIndexedCurve::ScaleCurve(float ScaleOrigin, float ScaleFactor)
{
	EnsureAllIndicesHaveHandles();

	TSet<FKeyHandle> KeyHandles;
	KeyHandles.Reserve(KeyHandlesToIndices.Num());

	for (auto It = KeyHandlesToIndices.CreateConstIterator(); It; ++It)
	{
		KeyHandles.Add(*It);
	}

	ScaleCurve(ScaleOrigin, ScaleFactor, KeyHandles);
}

void FIndexedCurve::ScaleCurve(float ScaleOrigin, float ScaleFactor, const TSet<FKeyHandle>& KeyHandles)
{
	if (KeyHandles.Num() != 0)
	{
		for (auto It = KeyHandles.CreateConstIterator(); It; ++It)
		{
			const FKeyHandle& KeyHandle = *It;
			if (KeyHandlesToIndices.Find(KeyHandle) != nullptr)
			{
				SetKeyTime(KeyHandle, (GetKeyTime(KeyHandle) - ScaleOrigin) * ScaleFactor + ScaleOrigin);
			}
		}
	}
}
