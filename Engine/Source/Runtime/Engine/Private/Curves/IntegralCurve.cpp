// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/IntegralCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IntegralCurve)

int32 FIntegralCurve::Evaluate(float Time, int32 InDefaultValue) const
{
	// If the default value hasn't been initialized, use the incoming default value
	int32 ReturnVal = DefaultValue == MAX_int32 ? InDefaultValue : DefaultValue;

	if (Keys.Num() == 0 || (bUseDefaultValueBeforeFirstKey && Time < Keys[0].Time))
	{
		// If no keys in curve, or bUseDefaultValueBeforeFirstKey is set and the time is before the first key, return the Default value.
	}
	else if (Keys.Num() < 2 || Time < Keys[0].Time)
	{
		// There is only one key or the time is before the first value. Return the first value
		ReturnVal = Keys[0].Value;
	}
	else if (Time < Keys[Keys.Num() - 1].Time)
	{
		// The key is in the range of Key[0] to Keys[Keys.Num()-1].  Find it by searching
		for (int32 i = 0; i < Keys.Num(); ++i)
		{
			if (Time < Keys[i].Time)
			{
				ReturnVal = Keys[FMath::Max(0, i - 1)].Value;
				break;
			}
		}
	}
	else
	{
		// Key is beyon the last point in the curve.  Return it's value
		ReturnVal = Keys[Keys.Num() - 1].Value;
	}

	return ReturnVal;
}


TArray<FIntegralKey>::TConstIterator FIntegralCurve::GetKeyIterator() const
{
	return Keys.CreateConstIterator();
}


FKeyHandle FIntegralCurve::AddKey(float InTime, int32 InValue, FKeyHandle InKeyHandle)
{
	int32 Index = 0;
	for (; Index < Keys.Num() && Keys[Index].Time < InTime; ++Index);
	Keys.Insert(FIntegralKey(InTime, InValue), Index);

	KeyHandlesToIndices.Add(InKeyHandle, Index);

	return GetKeyHandle(Index);
}


void FIntegralCurve::DeleteKey(FKeyHandle InKeyHandle)
{
	int32 Index = GetIndex(InKeyHandle);

	Keys.RemoveAt(Index);

	KeyHandlesToIndices.Remove(InKeyHandle);
}


FKeyHandle FIntegralCurve::UpdateOrAddKey(float InTime, int32 Value, float KeyTimeTolerance)
{
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		float KeyTime = Keys[KeyIndex].Time;

		if (FMath::IsNearlyEqual(KeyTime, InTime, KeyTimeTolerance))
		{
			Keys[KeyIndex].Value = Value;
			return GetKeyHandle(KeyIndex);
		}

		if (KeyTime > InTime)
		{
			// All the rest of the keys exist after the key we want to add
			// so there is no point in searching
			break;
		}
	}

	// A key wasnt found, add it now
	return AddKey(InTime, Value);
}


void FIntegralCurve::SetKeyTime(FKeyHandle KeyHandle, float NewTime)
{
	if (IsKeyHandleValid(KeyHandle))
	{
		const FIntegralKey OldKey = GetKey(KeyHandle);

		DeleteKey(KeyHandle);
		AddKey(NewTime, OldKey.Value, KeyHandle);

		// Copy all properties from old key, but then fix time to be the new time
		FIntegralKey& NewKey = GetKey(KeyHandle);
		NewKey = OldKey;
		NewKey.Time = NewTime;
	}
}


float FIntegralCurve::GetKeyTime(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Time;
}


void FIntegralCurve::SetKeyValue(FKeyHandle KeyHandle, int32 NewValue)
{
	if (IsKeyHandleValid(KeyHandle))
	{
		GetKey(KeyHandle).Value = NewValue;
	}
}


int32 FIntegralCurve::GetKeyValue(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Value;
}


FIntegralKey& FIntegralCurve::GetKey(FKeyHandle KeyHandle)
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}


FIntegralKey FIntegralCurve::GetKey(FKeyHandle KeyHandle) const
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}


FKeyHandle FIntegralCurve::FindKey(float KeyTime, float KeyTimeTolerance) const
{
	int32 Start = 0;
	int32 End = Keys.Num() - 1;

	// Binary search since the keys are in sorted order
	while (Start <= End)
	{
		int32 TestPos = Start + (End - Start) / 2;
		float TestKeyTime = Keys[TestPos].Time;

		if (FMath::IsNearlyEqual(TestKeyTime, KeyTime, KeyTimeTolerance))
		{
			return GetKeyHandle(TestPos);
		}
		else if (TestKeyTime < KeyTime)
		{
			Start = TestPos + 1;
		}
		else
		{
			End = TestPos - 1;
		}
	}

	return FKeyHandle::Invalid();
}

FKeyHandle FIntegralCurve::FindKeyBeforeOrAt(float KeyTime) const
{
	// If there are no keys or the time is before the first key return an invalid handle.
	if (Keys.Num() == 0 || KeyTime < Keys[0].Time)
	{
		return FKeyHandle::Invalid();
	}

	// If the time is after or at the last key return the last key.
	if (KeyTime >= Keys[Keys.Num() - 1].Time)
	{
		return GetKeyHandle(Keys.Num() - 1);
	}

	// Otherwise binary search to find the handle of the nearest key at or before the time.
	int32 Start = 0;
	int32 End = Keys.Num() - 1;
	int32 FoundIndex = -1;
	while (FoundIndex < 0)
	{
		int32 TestPos = (Start + End) / 2;
		float TestKeyTime = Keys[TestPos].Time;
		float NextTestKeyTime = Keys[TestPos + 1].Time;
		if (TestKeyTime <= KeyTime)
		{
			if (NextTestKeyTime > KeyTime)
			{
				FoundIndex = TestPos;
			}
			else
			{
				Start = TestPos + 1;
			}
		}
		else
		{
			End = TestPos;
		}
	}
	return GetKeyHandle(FoundIndex);
}

void FIntegralCurve::RemoveRedundantKeys()
{
	TArray<FKeyHandle> KeyHandlesToRemove;
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		if (KeyIndex + 2 < Keys.Num())
		{
			const FIntegralKey& CurrentKey = Keys[KeyIndex];
			const FIntegralKey& NextKeyOne = Keys[KeyIndex + 1];
			const FIntegralKey& NextKeyTwo = Keys[KeyIndex + 2];

			if (CurrentKey.Value == NextKeyOne.Value && NextKeyOne.Value == NextKeyTwo.Value)
			{
				KeyHandlesToRemove.Add(GetKeyHandle(KeyIndex + 1));
			}
		}
	}

	for (const FKeyHandle& Handle : KeyHandlesToRemove)
	{
		DeleteKey(Handle);
	}

	// Const value, remove keys and set default value
	if (Keys.Num() == 1)
	{
		DefaultValue = Keys[0].Value;
		DeleteKey(GetKeyHandle(0));
	}
}

