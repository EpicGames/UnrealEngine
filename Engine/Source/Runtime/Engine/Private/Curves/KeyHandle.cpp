// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/KeyHandle.h"
#include "AutoRTFM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KeyHandle)

// Always open around this CTOR as we always increment this atomic. If we abort we simply will just grab a new index when trying again
UE_AUTORTFM_ALWAYS_OPEN FKeyHandle::FKeyHandle()
{
	static std::atomic<uint32> LastKeyHandleIndex = 1;
	Index = ++LastKeyHandleIndex;

	if (Index == 0)
	{
		// If we are cooking, allow wrap-around
		if (IsRunningCookCommandlet())
		{
			// Skip indices until it's not 0 anymore as we can't 
			// assign without loss of thread-safety.
			while (Index == 0)
			{
				Index = ++LastKeyHandleIndex;
			}
		}
		else
		{
			check(Index != 0); // check in the unlikely event that this overflows
		}
	}
}

FKeyHandle::FKeyHandle(uint32 SpecificIndex)
	: Index(SpecificIndex)
{}

FKeyHandle FKeyHandle::Invalid()
{
	return FKeyHandle(0);
}

bool FKeyHandle::IsValid() const
{
	return Index != 0;
}

void FKeyHandleMap::Initialize(TArrayView<const FKeyHandle> InKeyHandles)
{
	Empty(InKeyHandles.Num());

	for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
	{
		const FKeyHandle& Handle = InKeyHandles[Index];
		KeyHandles.Add(Handle);
		KeyHandlesToIndices.Add(Handle, Index);
	}
}

/* FKeyHandleMap interface
 *****************************************************************************/

void FKeyHandleMap::Add( const FKeyHandle& InHandle, int32 InIndex )
{

	if (InIndex > KeyHandles.Num())
	{
		KeyHandles.Reserve(InIndex+1);
		for(int32 NewIndex = KeyHandles.Num(); NewIndex < InIndex; ++NewIndex)
		{
			KeyHandles.AddDefaulted();
			KeyHandlesToIndices.Add(KeyHandles.Last(), NewIndex);
		}
		KeyHandles.Add(InHandle);
	}
	else
	{
		if (InIndex < KeyHandles.Num())
		{
			for (auto It = KeyHandlesToIndices.CreateIterator(); It; ++It)
			{
				int32& KeyIndex = It.Value();
				if (KeyIndex >= InIndex) { ++KeyIndex; }
			}
		}
		KeyHandles.Insert(InHandle, InIndex);
	}

	KeyHandlesToIndices.Add(InHandle, InIndex);
}


void FKeyHandleMap::SetKeyHandles(int32 Num)
{
	KeyHandles.Reserve(Num);
	KeyHandlesToIndices.Reserve(Num);
	for (int32 Index = 0; Index < Num; ++Index)
	{
		FKeyHandle Handle;
		KeyHandles.Add(Handle);
		KeyHandlesToIndices.Add(Handle, Index);
	}
}


void FKeyHandleMap::Empty(int32 ExpectedNumElements)
{
	KeyHandlesToIndices.Empty(ExpectedNumElements);
	KeyHandles.Empty(ExpectedNumElements);
}


void FKeyHandleMap::Reserve(int32 NumElements)
{
	KeyHandlesToIndices.Reserve(NumElements);
	KeyHandles.Reserve(NumElements);
}


void FKeyHandleMap::Remove( const FKeyHandle& InHandle )
{
	int32 Index = INDEX_NONE;
	if (KeyHandlesToIndices.RemoveAndCopyValue(InHandle, Index))
	{
		// update key indices
		for (auto It = KeyHandlesToIndices.CreateIterator(); It; ++It)
		{
			int32& KeyIndex = It.Value();
			if (KeyIndex >= Index) { --KeyIndex; }
		}

		KeyHandles.RemoveAt(Index);
	}
}

const FKeyHandle* FKeyHandleMap::FindKey( int32 KeyIndex ) const
{
	if (KeyIndex >= 0 && KeyIndex < KeyHandles.Num())
	{
		return &KeyHandles[KeyIndex];
	}
	return nullptr;
}

bool FKeyHandleMap::Serialize(FArchive& Ar)
{
	// only allow this map to be saved to the transaction buffer
	if( Ar.IsTransacting() )
	{
		Ar << KeyHandlesToIndices;
		Ar << KeyHandles;
	}

	return true;
}

void FKeyHandleMap::EnsureAllIndicesHaveHandles(int32 NumIndices)
{
	if (KeyHandles.Num() > NumIndices)
	{
		for (int32 Index = NumIndices; Index < KeyHandles.Num(); ++Index)
		{
			KeyHandlesToIndices.Remove(KeyHandles[Index]);
		}

		KeyHandles.SetNum(NumIndices);
	}
	else if (KeyHandles.Num() < NumIndices)
	{
		KeyHandles.Reserve(NumIndices);
		for (int32 NewIndex = KeyHandles.Num(); NewIndex < NumIndices; ++NewIndex)
		{
			KeyHandles.AddDefaulted();
			KeyHandlesToIndices.Add(KeyHandles.Last(), NewIndex);
		}
	}
}

void FKeyHandleMap::EnsureIndexHasAHandle(int32 KeyIndex)
{
	const FKeyHandle* KeyHandle = FindKey(KeyIndex);
	if (!KeyHandle)
	{
		Add(FKeyHandle(), KeyIndex);
	}
}

int32 FKeyHandleLookupTable::GetIndex(FKeyHandle KeyHandle)
{
	const int32* Index = KeyHandlesToIndices.Find(KeyHandle);
	return Index ? *Index : INDEX_NONE;
}

FKeyHandle FKeyHandleLookupTable::FindOrAddKeyHandle(int32 Index)
{
	if (KeyHandles.IsValidIndex(Index))
	{
		return KeyHandles[Index];
	}

	// Allocate a new key handle
	FKeyHandle NewKeyHandle;

	KeyHandles.Insert(Index, NewKeyHandle);
	KeyHandlesToIndices.Add(NewKeyHandle, Index);

	return NewKeyHandle;
}

void FKeyHandleLookupTable::MoveHandle(int32 OldIndex, int32 NewIndex)
{
	if (KeyHandles.IsValidIndex(OldIndex))
	{
		FKeyHandle Handle = KeyHandles[OldIndex];

		KeyHandles.RemoveAt(OldIndex);
		RelocateKeyHandles(OldIndex, -1);

		if (NewIndex < KeyHandles.GetMaxIndex())
		{
			// Move proceeding keys forward to make space
			RelocateKeyHandles(NewIndex, 1);
		}

		KeyHandles.Insert(NewIndex, Handle);
		KeyHandlesToIndices.Add(Handle, NewIndex);
	}
}

FKeyHandle FKeyHandleLookupTable::AllocateHandle(int32 Index)
{
	FKeyHandle NewKeyHandle;

	if (Index < KeyHandles.GetMaxIndex())
	{
		// Move proceeding keys forward to make space
		RelocateKeyHandles(Index, 1);
	}

	KeyHandles.Insert(Index, NewKeyHandle);
	KeyHandlesToIndices.Add(NewKeyHandle, Index);
	return NewKeyHandle;
}

void FKeyHandleLookupTable::DeallocateHandle(int32 Index)
{
	if (KeyHandles.IsValidIndex(Index))
	{
		KeyHandlesToIndices.Remove(KeyHandles[Index]);
		KeyHandles.RemoveAt(Index, 1);

		// Move proceeding keys into the gap we just made
		RelocateKeyHandles(Index+1, -1);
	}
}

void FKeyHandleLookupTable::ReplaceHandle(int32 Index, FKeyHandle HandleToReplace)
{
	if (KeyHandles.IsValidIndex(Index) && !KeyHandlesToIndices.Contains(HandleToReplace))
	{
		KeyHandlesToIndices.Remove(KeyHandles[Index]);
		KeyHandlesToIndices.Add(HandleToReplace, Index);
		KeyHandles[Index] = HandleToReplace;
	}
}

void FKeyHandleLookupTable::Reset()
{
	KeyHandles.Reset();
	KeyHandlesToIndices.Reset();
}

bool FKeyHandleLookupTable::Serialize(FArchive& Ar)
{
	// We're only concerned with Undo/Redo transactions
	if (Ar.IsTransacting())
	{
		// Serialize the sparse array so as to preserve indices (by default it only serializes valid entries)
		int32 NumHandles = KeyHandles.GetMaxIndex();
		Ar << NumHandles;

		FKeyHandle InvalidHandle = FKeyHandle::Invalid();

		if (Ar.IsLoading())
		{
			KeyHandles.Empty(NumHandles);
			for (int32 Index = 0; Index < NumHandles; ++Index)
			{
				FKeyHandle Handle = InvalidHandle;
				Ar << Handle;
				if (Handle != InvalidHandle)
				{
					KeyHandles.Insert(Index, Handle);
				}
			}
		}
		else if (Ar.IsSaving())
		{
			for (int32 Index = 0; Index < NumHandles; ++Index)
			{
				FKeyHandle Handle = KeyHandles.IsAllocated(Index) ? KeyHandles[Index] : InvalidHandle;
				Ar << Handle;
			}
		}

		Ar << KeyHandlesToIndices;
	}

	return true;
}

void FKeyHandleLookupTable::RelocateKeyHandles(int32 StartAtIndex, int32 DeltaIndex)
{
	if (DeltaIndex == 0)
	{
		return;
	}

	const int32 OldNumKeys = KeyHandles.GetMaxIndex();
	const int32 NewNumKeys = KeyHandles.GetMaxIndex() + DeltaIndex;

	// Iterate forwards when removing elements, and backwards when adding elements
	const int32 StartIndex = DeltaIndex > 0 ? OldNumKeys-1   : StartAtIndex;
	const int32 EndIndex   = DeltaIndex > 0 ? StartAtIndex-1 : OldNumKeys;
	const int32 Inc        = DeltaIndex > 0 ? -1             : 1;

	if (DeltaIndex > 0)
	{
		// Reserve enough space for the new elements
		KeyHandles.Reserve(NewNumKeys);
	}

	// Move handles and fixup indices
	for (int32 FixupIndex = StartIndex; FixupIndex != EndIndex; FixupIndex += Inc)
	{
		if (KeyHandles.IsAllocated(FixupIndex))
		{
			FKeyHandle Handle = KeyHandles[FixupIndex];

			KeyHandles.Insert(FixupIndex+DeltaIndex, Handle);
			KeyHandlesToIndices.Add(Handle, FixupIndex+DeltaIndex);

			KeyHandles.RemoveAt(FixupIndex);
		}
	}

	if (DeltaIndex < 0)
	{
		KeyHandles.Shrink();
	}
}
