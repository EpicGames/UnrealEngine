// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/LinearColorRamp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LinearColorRamp)

namespace UE::LinearColorMap::Private
{
	static const FName RedCurveName("R");
	static const FName GreenCurveName("G");
	static const FName BlueCurveName("B");
	static const FName AlphaCurveName("A");
}

////////////////////////////////////////////////////////////////////////////////

void FLinearColorRamp::SetFrom(const FLinearColorRamp& Other)
{
	RichCurves[0] = Other.RichCurves[0];
	RichCurves[1] = Other.RichCurves[1];
	RichCurves[2] = Other.RichCurves[2];
	RichCurves[3] = Other.RichCurves[3];
	OnCurveChanged(GetCurves());
}


bool FLinearColorRamp::IsEmpty() const
{
	return (RichCurves[0].IsEmpty()
		&& RichCurves[1].IsEmpty()
		&& RichCurves[2].IsEmpty()
		&& RichCurves[3].IsEmpty());
}

void FLinearColorRamp::Reset()
{
	RichCurves[0].Reset();
	RichCurves[1].Reset();
	RichCurves[2].Reset();
	RichCurves[3].Reset();
}

void FLinearColorRamp::SetColorAtTime(float Time, const FLinearColor& Color, bool bOnlyRBG)
{
	RichCurves[0].AddKey(Time, Color.R);
	RichCurves[1].AddKey(Time, Color.G);
	RichCurves[2].AddKey(Time, Color.B);
	if (!bOnlyRBG)
	{
		RichCurves[3].AddKey(Time, Color.A);
	}
}

TArray<FRichCurveEditInfoConst> FLinearColorRamp::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[0], UE::LinearColorMap::Private::RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[1], UE::LinearColorMap::Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[2], UE::LinearColorMap::Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[3], UE::LinearColorMap::Private::AlphaCurveName));
	return Curves;
}

void FLinearColorRamp::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[0],UE::LinearColorMap::Private::RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[1],UE::LinearColorMap::Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[2],UE::LinearColorMap::Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[3],UE::LinearColorMap::Private::AlphaCurveName));
}

TArray<FRichCurveEditInfo> FLinearColorRamp::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(FRichCurveEditInfo(&RichCurves[0], UE::LinearColorMap::Private::RedCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[1], UE::LinearColorMap::Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[2], UE::LinearColorMap::Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[3], UE::LinearColorMap::Private::AlphaCurveName));
	return Curves;
}

void FLinearColorRamp::ModifyOwner()
{
}

TArray<const UObject*> FLinearColorRamp::GetOwners() const
{
	static TArray<const UObject*> EmptyObjectArray;
	return EmptyObjectArray;
}

void FLinearColorRamp::MakeTransactional()
{
}

void FLinearColorRamp::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	TArray<FRichCurve*> ChangedCurves;
	for (const FRichCurveEditInfo& ChangedCurveEditInfo : ChangedCurveEditInfos)
	{
		ChangedCurves.Add((FRichCurve*)ChangedCurveEditInfo.CurveToEdit);
	}
	if (ChangedCurves.Num() > 0)
	{
		OnColorCurveChangedDelegate.Broadcast(ChangedCurves);
	}
}

bool FLinearColorRamp::IsLinearColorCurve() const
{
	return true;
}

FLinearColor FLinearColorRamp::GetLinearColorValue(float InTime) const
{
	return FLinearColor(
		RichCurves[0].Eval(InTime),
		RichCurves[1].Eval(InTime),
		RichCurves[2].Eval(InTime),
		RichCurves[3].Eval(InTime));
}

bool FLinearColorRamp::HasAnyAlphaKeys() const
{
	return RichCurves[3].GetNumKeys() != 0;
}

bool FLinearColorRamp::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	return (CurveInfo.CurveToEdit == &RichCurves[0]
		|| CurveInfo.CurveToEdit == &RichCurves[1]
		|| CurveInfo.CurveToEdit == &RichCurves[2]
		|| CurveInfo.CurveToEdit == &RichCurves[3]);
}

FLinearColor FLinearColorRamp::GetCurveColor(FRichCurveEditInfo CurveInfo) const
{
	if (CurveInfo.CurveToEdit == &RichCurves[0])
	{
		return FLinearColor::Red;
	}
	else if (CurveInfo.CurveToEdit == &RichCurves[1])
	{
		return FLinearColor::Green;
	}
	else if (CurveInfo.CurveToEdit == &RichCurves[2])
	{
		return FLinearColor::Blue;
	}
	return FLinearColor::White;
}
