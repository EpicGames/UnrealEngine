// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/NameCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NameCurve)


/* FNameCurveKey interface
 *****************************************************************************/

bool FNameCurveKey::operator==(const FNameCurveKey& Curve) const
{
	return ((Time == Curve.Time) && (Value == Curve.Value));
}


bool FNameCurveKey::operator!=(const FNameCurveKey& Other) const
{
	return !(*this == Other);
}


bool FNameCurveKey::Serialize(FArchive& Ar)
{
	Ar << Time << Value;
	return true;
}


/* FNameCurve interface
 *****************************************************************************/

FKeyHandle FNameCurve::AddKey(float InTime, const FName& InValue, FKeyHandle KeyHandle)
{
	int32 Index = 0;

	// insert key
	for(; Index < Keys.Num() && Keys[Index].Time < InTime; ++Index);
	Keys.Insert(FNameCurveKey(InTime, InValue), Index);

	KeyHandlesToIndices.Add(KeyHandle, Index);

	return GetKeyHandle(Index);
}


void FNameCurve::DeleteKey(FKeyHandle KeyHandle)
{
	// remove key
	int32 Index = GetIndex(KeyHandle);
	Keys.RemoveAt(Index);

	KeyHandlesToIndices.Remove(KeyHandle);
}


FKeyHandle FNameCurve::FindKey(float KeyTime, float KeyTimeTolerance) const
{
	int32 Start = 0;
	int32 End = Keys.Num()-1;

	// Binary search since the keys are in sorted order
	while (Start <= End)
	{
		int32 TestPos = Start + (End-Start) / 2;
		float TestKeyTime = Keys[TestPos].Time;

		if (FMath::IsNearlyEqual(TestKeyTime, KeyTime, KeyTimeTolerance))
		{
			return GetKeyHandle(TestPos);
		}

		if (TestKeyTime < KeyTime)
		{
			Start = TestPos+1;
		}
		else
		{
			End = TestPos-1;
		}
	}

	return FKeyHandle::Invalid();
}


FNameCurveKey& FNameCurve::GetKey(FKeyHandle KeyHandle)
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}


FNameCurveKey FNameCurve::GetKey(FKeyHandle KeyHandle) const
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}


float FNameCurve::GetKeyTime(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Time;
}


void FNameCurve::SetKeyTime(FKeyHandle KeyHandle, float NewTime)
{
	if (IsKeyHandleValid(KeyHandle))
	{
		const FNameCurveKey OldKey = GetKey(KeyHandle);

		DeleteKey(KeyHandle);
		AddKey(NewTime, OldKey.Value, KeyHandle);

		// Copy all properties from old key, but then fix time to be the new time
		FNameCurveKey& NewKey = GetKey(KeyHandle);
		NewKey = OldKey;
		NewKey.Time = NewTime;
	}
}

FKeyHandle FNameCurve::UpdateOrAddKey(float InTime, const FName& InValue, float KeyTimeTolerance)
{
	// Search for a key that already exists at the time and if found, update its value
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		float KeyTime = Keys[KeyIndex].Time;

		if (FMath::IsNearlyEqual(KeyTime, InTime, KeyTimeTolerance))
		{
			Keys[KeyIndex].Value = InValue;

			return GetKeyHandle(KeyIndex);
		}

		if (KeyTime > InTime)
		{
			// All the rest of the keys exist after the key we want to add
			// so there is no point in searching
			break;
		}
	}

	// A key wasnt found, add it now
	return AddKey(InTime, InValue);
}

