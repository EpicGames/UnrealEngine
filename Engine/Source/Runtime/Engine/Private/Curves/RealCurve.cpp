// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/RealCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RealCurve)

void FRealCurve::CycleTime(float MinTime, float MaxTime, float& InTime, int& CycleCount)
{
	float InitTime = InTime;
	float Duration = MaxTime - MinTime;

	if (InTime > MaxTime)
	{
		CycleCount = FMath::FloorToInt((MaxTime - InTime) / Duration);
		InTime = InTime + Duration * CycleCount;
	}
	else if (InTime < MinTime)
	{
		CycleCount = FMath::FloorToInt((InTime - MinTime) / Duration);
		InTime = InTime - Duration * CycleCount;
	}

	if (InTime == MaxTime && InitTime < MinTime)
	{
		InTime = MinTime;
	}

	if (InTime == MinTime && InitTime > MaxTime)
	{
		InTime = MaxTime;
	}

	CycleCount = FMath::Abs(CycleCount);
}

FKeyHandle FRealCurve::FindKey(float KeyTime, float KeyTimeTolerance) const
{
	const int32 KeyIndex = GetKeyIndex(KeyTime, KeyTimeTolerance);
	if (KeyIndex >= 0)
	{
		return GetKeyHandle(KeyIndex);
	}

	return FKeyHandle::Invalid();
}

bool FRealCurve::KeyExistsAtTime(float KeyTime, float KeyTimeTolerance) const
{
	return GetKeyIndex(KeyTime, KeyTimeTolerance) >= 0;
}
