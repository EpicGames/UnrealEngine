// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/RichCurve.h"
#include "Curves/CurveEvaluation.h"
#include "UObject/UE5MainStreamObjectVersion.h"

// Broken - do not turn on! 

#include UE_INLINE_GENERATED_CPP_BY_NAME(RichCurve)
#define MIXEDKEY_STRIPS_TANGENTS 0

/* FRichCurveKey interface
 *****************************************************************************/

static void SetModesFromLegacy(FRichCurveKey& InKey, EInterpCurveMode InterpMode)
{
	InKey.InterpMode = RCIM_Linear;
	InKey.TangentWeightMode = RCTWM_WeightedNone;
	InKey.TangentMode = RCTM_Auto;

	if (InterpMode == CIM_Constant)
	{
		InKey.InterpMode = RCIM_Constant;
	}
	else if (InterpMode == CIM_Linear)
	{
		InKey.InterpMode = RCIM_Linear;
	}
	else
	{
		InKey.InterpMode = RCIM_Cubic;

		if (InterpMode == CIM_CurveAuto || InterpMode == CIM_CurveAutoClamped)
		{
			InKey.TangentMode = RCTM_Auto;
		}
		else if (InterpMode == CIM_CurveBreak)
		{
			InKey.TangentMode = RCTM_Break;
		}
		else if (InterpMode == CIM_CurveUser)
		{
			InKey.TangentMode = RCTM_User;
		}
	}
}


FRichCurveKey::FRichCurveKey(const FInterpCurvePoint<float>& InPoint)
{
	SetModesFromLegacy(*this, InPoint.InterpMode);

	Time = InPoint.InVal;
	Value = InPoint.OutVal;

	ArriveTangent = InPoint.ArriveTangent;
	ArriveTangentWeight = 0.f;

	LeaveTangent = InPoint.LeaveTangent;
	LeaveTangentWeight = 0.f;
}

FRichCurveKey::FRichCurveKey(const FInterpCurvePoint<FVector2D>& InPoint, int32 ComponentIndex)
{
	SetModesFromLegacy(*this, InPoint.InterpMode);

	Time = InPoint.InVal;

	if (ComponentIndex == 0)
	{
		Value = InPoint.OutVal.X;
		ArriveTangent = InPoint.ArriveTangent.X;
		LeaveTangent = InPoint.LeaveTangent.X;
	}
	else
	{
		Value = InPoint.OutVal.Y;
		ArriveTangent = InPoint.ArriveTangent.Y;
		LeaveTangent = InPoint.LeaveTangent.Y;
	}


	ArriveTangentWeight = 0.f;
	LeaveTangentWeight = 0.f;
}

FRichCurveKey::FRichCurveKey(const FInterpCurvePoint<FVector>& InPoint, int32 ComponentIndex)
{
	SetModesFromLegacy(*this, InPoint.InterpMode);

	Time = InPoint.InVal;

	if (ComponentIndex == 0)
	{
		Value = InPoint.OutVal.X;
		ArriveTangent = InPoint.ArriveTangent.X;
		LeaveTangent = InPoint.LeaveTangent.X;
	}
	else if (ComponentIndex == 1)
	{
		Value = InPoint.OutVal.Y;
		ArriveTangent = InPoint.ArriveTangent.Y;
		LeaveTangent = InPoint.LeaveTangent.Y;
	}
	else
	{
		Value = InPoint.OutVal.Z;
		ArriveTangent = InPoint.ArriveTangent.Z;
		LeaveTangent = InPoint.LeaveTangent.Z;
	}

	ArriveTangentWeight = 0.f;
	LeaveTangentWeight = 0.f;
}

FRichCurveKey::FRichCurveKey(const FInterpCurvePoint<FTwoVectors>& InPoint, int32 ComponentIndex)
{
	SetModesFromLegacy(*this, InPoint.InterpMode);

	Time = InPoint.InVal;

	if (ComponentIndex == 0)
	{
		Value = InPoint.OutVal.v1.X;
		ArriveTangent = InPoint.ArriveTangent.v1.X;
		LeaveTangent = InPoint.LeaveTangent.v1.X;
	}
	else if (ComponentIndex == 1)
	{
		Value = InPoint.OutVal.v1.Y;
		ArriveTangent = InPoint.ArriveTangent.v1.Y;
		LeaveTangent = InPoint.LeaveTangent.v1.Y;
	}
	else if (ComponentIndex == 2)
	{
		Value = InPoint.OutVal.v1.Z;
		ArriveTangent = InPoint.ArriveTangent.v1.Z;
		LeaveTangent = InPoint.LeaveTangent.v1.Z;
	}
	else if (ComponentIndex == 3)
	{
		Value = InPoint.OutVal.v2.X;
		ArriveTangent = InPoint.ArriveTangent.v2.X;
		LeaveTangent = InPoint.LeaveTangent.v2.X;
	}
	else if (ComponentIndex == 4)
	{
		Value = InPoint.OutVal.v2.Y;
		ArriveTangent = InPoint.ArriveTangent.v2.Y;
		LeaveTangent = InPoint.LeaveTangent.v2.Y;
	}
	else
	{
		Value = InPoint.OutVal.v2.Z;
		ArriveTangent = InPoint.ArriveTangent.v2.Z;
		LeaveTangent = InPoint.LeaveTangent.v2.Z;
	}

	ArriveTangentWeight = 0.f;
	LeaveTangentWeight = 0.f;
}

bool FRichCurveKey::Serialize(FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_SERIALIZE_RICH_CURVE_KEY)
	{
		return false;
	}

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	// Serialization is handled manually to avoid the extra size overhead of FProperty tagging.
	// Otherwise with many keys in a rich curve the size can become quite large.
	Ar << InterpMode;
	Ar << TangentMode;
	Ar << TangentWeightMode;
	Ar << Time;
	Ar << Value;
	Ar << ArriveTangent;
	Ar << ArriveTangentWeight;
	Ar << LeaveTangent;
	Ar << LeaveTangentWeight;

	if (Ar.IsLoading())
	{
		if (TangentWeightMode != RCTWM_WeightedNone &&
			TangentWeightMode != RCTWM_WeightedArrive &&
			TangentWeightMode != RCTWM_WeightedLeave &&
			TangentWeightMode != RCTWM_WeightedBoth)
		{
			if(Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RichCurveKeyInvalidTangentMode)
			{
				TangentWeightMode = RCTWM_WeightedNone;	

				// No valid weight mode - find correct one one
				const bool bHasArriveWeight = !FMath::IsNearlyZero(ArriveTangentWeight);
				const bool bHasLeaveWeight = !FMath::IsNearlyZero(LeaveTangentWeight);

				if (bHasArriveWeight && bHasLeaveWeight)
				{
					TangentWeightMode = RCTWM_WeightedBoth;
				}
				else if (bHasArriveWeight)
				{
					TangentWeightMode = RCTWM_WeightedArrive;
				}
				else if (bHasLeaveWeight)
				{
					TangentWeightMode = RCTWM_WeightedLeave;
				}
			}
			else
			{
				ensureMsgf(false, TEXT("FRichCurveKey found with invalid TangentWeightMode value of %i"), static_cast<int32>(TangentWeightMode));
			}			
		}
	}

	return true;
}

bool FRichCurveKey::operator==( const FRichCurveKey& Curve ) const
{
	return (Time == Curve.Time) && (Value == Curve.Value) && (InterpMode == Curve.InterpMode) &&
		   (TangentMode == Curve.TangentMode) && (TangentWeightMode == Curve.TangentWeightMode) &&
		   ((InterpMode != RCIM_Cubic) || //also verify if it is cubic that tangents are the same
		   ((ArriveTangent == Curve.ArriveTangent) && (LeaveTangent == Curve.LeaveTangent) ));

}


bool FRichCurveKey::operator!=(const FRichCurveKey& Other) const
{
	return !(*this == Other);
}

/* FRichCurve interface
 *****************************************************************************/

TArray<FRichCurveKey> FRichCurve::GetCopyOfKeys() const
{
	return Keys;
}

const TArray<FRichCurveKey>& FRichCurve::GetConstRefOfKeys() const
{
	return Keys;
}


TArray<FRichCurveKey>::TConstIterator FRichCurve::GetKeyIterator() const
{
	return Keys.CreateConstIterator();
}


FRichCurveKey& FRichCurve::GetKey(FKeyHandle KeyHandle)
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}


FRichCurveKey FRichCurve::GetKey(FKeyHandle KeyHandle) const
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}

const FRichCurveKey& FRichCurve::GetKeyRef(FKeyHandle KeyHandle) const
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}

FRichCurveKey FRichCurve::GetFirstKey() const
{
	check(Keys.Num() > 0);
	return Keys[0];
}


FRichCurveKey FRichCurve::GetLastKey() const
{
	check(Keys.Num() > 0);
	return Keys[Keys.Num()-1];
}


FRichCurveKey* FRichCurve::GetFirstMatchingKey(const TArray<FKeyHandle>& KeyHandles)
{
	for (const auto& KeyHandle : KeyHandles)
	{
		if (IsKeyHandleValid(KeyHandle))
		{
			return &GetKey(KeyHandle);
		}
	}

	return nullptr;
}

FKeyHandle FRichCurve::AddKey( const float InTime, const float InValue, const bool bUnwindRotation, FKeyHandle NewHandle )
{
	int32 Index = 0;
	if (Keys.Num() == 0 || Keys.Last().Time < InTime)
	{
		Index = Keys.Num();
	}
	else
	{
		for (; Index < Keys.Num() && Keys[Index].Time < InTime; ++Index);
	}
	Keys.Insert(FRichCurveKey(InTime, InValue), Index);

	// If we were asked to treat this curve as a rotation value and to unwindow the rotation, then
	// we'll look at the previous key and modify the key's value to use a rotation angle that is
	// continuous with the previous key while retaining the exact same rotation angle, if at all necessary
	if( Index > 0 && bUnwindRotation )
	{
		const float OldValue = Keys[ Index - 1 ].Value;
		float NewValue = Keys[ Index ].Value;

		while( NewValue - OldValue > 180.0f )
		{
			NewValue -= 360.0f;
		}
		while( NewValue - OldValue < -180.0f )
		{
			NewValue += 360.0f;
		}

		Keys[Index].Value = NewValue;
	}
	
	KeyHandlesToIndices.Add(NewHandle, Index);

	return NewHandle;
}


void FRichCurve::ReserveKeys(const int32 Number)
{
	Keys.Reserve(Number);
	KeyHandlesToIndices.Reserve(Number);
}


void FRichCurve::SetKeys(const TArray<FRichCurveKey>& InKeys)
{
	Reset();

	Keys.SetNum(InKeys.Num());
	KeyHandlesToIndices.SetKeyHandles(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Keys[Index] = InKeys[Index];
	}

	AutoSetTangents();
}


void FRichCurve::DeleteKey(FKeyHandle InKeyHandle)
{
	int32 Index = GetIndex(InKeyHandle);
	
	Keys.RemoveAt(Index);
	AutoSetTangents();

	KeyHandlesToIndices.Remove(InKeyHandle);
}


FKeyHandle FRichCurve::UpdateOrAddKey(float InTime, float InValue, const bool bUnwindRotation, float KeyTimeTolerance)
{
	// Search for a key that already exists at the time and if found, update its value
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		float KeyTime = Keys[KeyIndex].Time;

		if (FMath::IsNearlyEqual(KeyTime, InTime, KeyTimeTolerance))
		{
			Keys[KeyIndex].Value = InValue;

			return GetKeyHandle(KeyIndex);
		}

		if (KeyTime > InTime)
		{
			// All the rest of the keys exist after the key we want to add
			// so there is no point in searching
			break;
		}
	}

	// A key wasn't found, add it now
	return AddKey(InTime, InValue, bUnwindRotation);
}


void FRichCurve::SetKeyTime( FKeyHandle KeyHandle, float NewTime )
{
	if (IsKeyHandleValid(KeyHandle))
	{
		const FRichCurveKey OldKey = GetKey(KeyHandle);

		DeleteKey(KeyHandle);
		AddKey(NewTime, OldKey.Value, false, KeyHandle);

		// Copy all properties from old key, but then fix time to be the new time
		FRichCurveKey& NewKey = GetKey(KeyHandle);
		NewKey = OldKey;
		NewKey.Time = NewTime;
	}
}


float FRichCurve::GetKeyTime(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Time;
}


int32 FRichCurve::GetKeyIndex(float KeyTime, float KeyTimeTolerance) const
{
	int32 Start = 0;
	int32 End = Keys.Num() - 1;

	// Binary search since the keys are in sorted order
	while (Start <= End)
	{
		int32 TestPos = Start + (End - Start) / 2;
		float TestKeyTime = Keys[TestPos].Time;

		if (FMath::IsNearlyEqual(TestKeyTime, KeyTime, KeyTimeTolerance))
		{
			return TestPos;
		}
		else if (TestKeyTime < KeyTime)
		{
			Start = TestPos + 1;
		}
		else
		{
			End = TestPos - 1;
		}
	}

	return INDEX_NONE;
}

void FRichCurve::SetKeyValue(FKeyHandle KeyHandle, float NewValue, bool bAutoSetTangents)
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return;
	}

	GetKey(KeyHandle).Value = NewValue;

	if (bAutoSetTangents)
	{
		AutoSetTangents();
	}
}


float FRichCurve::GetKeyValue(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Value;
}

bool FRichCurve::IsConstant(float Tolerance) const
{
	if (Keys.Num() <= 1)
	{
		return true;
	}

	const FRichCurveKey& RefKey = Keys[0];
	for (const FRichCurveKey& Key : Keys)
	{
		if (!FMath::IsNearlyEqual(Key.Value, RefKey.Value, Tolerance) || Key.InterpMode == ERichCurveInterpMode::RCIM_Cubic)
		{
			return false;
		}
	}

	return true;
}

TPair<float, float> FRichCurve::GetKeyTimeValuePair(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return TPair<float, float>(0.f, 0.f);
	}

	const FRichCurveKey& Key = GetKey(KeyHandle);
	return TPair<float, float>(Key.Time, Key.Value);
}

void FRichCurve::SetKeyInterpMode(FKeyHandle KeyHandle, ERichCurveInterpMode NewInterpMode)
{
	SetKeyInterpMode(KeyHandle, NewInterpMode, true);
}

void FRichCurve::SetKeyInterpMode(FKeyHandle KeyHandle, ERichCurveInterpMode NewInterpMode, bool bAutoSetTangents)
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return;
	}

	GetKey(KeyHandle).InterpMode = NewInterpMode;
	if (bAutoSetTangents)
	{
		AutoSetTangents();
	}
}


void FRichCurve::SetKeyTangentMode(FKeyHandle KeyHandle, ERichCurveTangentMode NewTangentMode, bool bAutoSetTangents /*= true*/)
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return;
	}

	GetKey(KeyHandle).TangentMode = NewTangentMode;
	if (bAutoSetTangents)
	{
		AutoSetTangents();
	}
}


void FRichCurve::SetKeyTangentWeightMode(FKeyHandle KeyHandle, ERichCurveTangentWeightMode NewTangentWeightMode, bool bAutoSetTangents /*= true*/)
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return;
	}

	GetKey(KeyHandle).TangentWeightMode = NewTangentWeightMode;
	if (bAutoSetTangents)
	{
		AutoSetTangents();
	}
}


ERichCurveInterpMode FRichCurve::GetKeyInterpMode(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return RCIM_Linear;
	}

	return GetKey(KeyHandle).InterpMode;
}


ERichCurveTangentMode FRichCurve::GetKeyTangentMode(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return RCTM_Auto;
	}

	return GetKey(KeyHandle).TangentMode;
}


void FRichCurve::GetTimeRange(float& MinTime, float& MaxTime) const
{
	if (Keys.Num() == 0)
	{
		MinTime = 0.f;
		MaxTime = 0.f;
	}
	else
	{
		MinTime = Keys[0].Time;
		MaxTime = Keys[Keys.Num()-1].Time;
	}
}


/*	 Finds min/max for cubic curves:
	Looks for feature points in the signal(determined by change in direction of local tangent), these locations are then re-examined in closer detail recursively */
template<class T>
void FeaturePointMethod(T& Function , float StartTime,  float EndTime, float StartValue,float Mu, int Depth, int MaxDepth, float& MaxV, float& MinVal)
{
	if (Depth >= MaxDepth)
	{
		return;
	}

	float PrevValue   = StartValue;
	float PrevTangent = StartValue - Function.Eval(StartTime-Mu);
	EndTime += Mu;

	// Make sure to try and keep our iteration floats within a similar range to avoid incrementing a large float with a small one leading
	// to an infinite loop
	const float RelativeEndTime = EndTime - StartTime;
	for (float f = Mu; f < RelativeEndTime; f += Mu)
	{
		const float CurrentTime = StartTime + f;
		float Value		 = Function.Eval(CurrentTime);

		MaxV   = FMath::Max(Value, MaxV);
		MinVal = FMath::Min(Value, MinVal);

		float CurTangent = Value - PrevValue;

		//Change direction? Examine this area closer
		if (FMath::Sign(CurTangent) != FMath::Sign(PrevTangent))
		{
			//feature point centered around the previous tangent
			float FeaturePointTime = CurrentTime-Mu*2.0f;
			FeaturePointMethod(Function, FeaturePointTime, CurrentTime, Function.Eval(FeaturePointTime), Mu*0.4f,Depth+1, MaxDepth, MaxV, MinVal);
		}

		PrevTangent = CurTangent;
		PrevValue = Value;
	}
}


void FRichCurve::GetValueRange(float& MinValue, float& MaxValue) const
{
	if (Keys.Num() == 0)
	{
		MinValue = MaxValue = 0.f;
	}
	else
	{
		int32 LastKeyIndex = Keys.Num()-1;
		MinValue = MaxValue = Keys[0].Value;

		for (int32 i = 0; i < Keys.Num(); i++)
		{
			const FRichCurveKey& Key = Keys[i];

			MinValue = FMath::Min(MinValue, Key.Value);
			MaxValue = FMath::Max(MaxValue, Key.Value);

			if (Key.InterpMode == RCIM_Cubic && i != LastKeyIndex)
			{
				const FRichCurveKey& NextKey = Keys[i+1];
				float TimeStep = (NextKey.Time - Key.Time) * 0.2f;

				FeaturePointMethod(*this, Key.Time, NextKey.Time, Key.Value, TimeStep, 0, 3, MaxValue, MinValue);
			}
		}
	}
}


void FRichCurve::Reset()
{
	Keys.Empty();
	KeyHandlesToIndices.Empty();
}


void FRichCurve::AutoSetTangents(float Tension)
{
	// Iterate over all points in this InterpCurve
	for (int32 KeyIndex = 0; KeyIndex<Keys.Num(); KeyIndex++)
	{
		FRichCurveKey& Key = Keys[KeyIndex];
		float ArriveTangent = Key.ArriveTangent;
		float LeaveTangent  = Key.LeaveTangent;

		// Variables for computing tangent.
		bool bNeedsComputeTangent = false;
		const FRichCurveKey* PrevKey = &Key;
		const FRichCurveKey* CurrentKey = &Key;
		const FRichCurveKey* NextKey = &Key;

		if (KeyIndex == 0) // Start point
		{
			// If first section is not a curve, or is a curve and first point has manual tangent setting.
			if (KeyIndex < (Keys.Num() - 1) && Key.TangentMode == RCTM_Auto)
			{
				LeaveTangent = 0.0f;
			}
		}
		else
		{
			if (KeyIndex < Keys.Num() - 1) // Inner point
			{
				PrevKey = &Keys[KeyIndex-1];

				if (Key.InterpMode == RCIM_Cubic && (Key.TangentMode == RCTM_Auto))
				{
					NextKey = &Keys[KeyIndex+1];
					bNeedsComputeTangent = true;
				}
				else if ((PrevKey->InterpMode == RCIM_Constant) || (Key.InterpMode == RCIM_Constant))
				{
					LeaveTangent  = 0.0f;

					if (PrevKey->InterpMode != RCIM_Cubic)
					{
						ArriveTangent = 0.0f;
					}
				}
				
			}
			else // End point
			{
				// If last section is not a curve, or is a curve and final point has manual tangent setting.
				if (Key.InterpMode == RCIM_Cubic && Key.TangentMode == RCTM_Auto)
				{
					ArriveTangent = 0.0f;
				}
			}
		}

		if (bNeedsComputeTangent)
		{
			ComputeCurveTangent(
				PrevKey->Time,		// Previous time
				PrevKey->Value,		// Previous point
				CurrentKey->Time,	// Current time
				CurrentKey->Value,	// Current point
				NextKey->Time,		// Next time
				NextKey->Value,		// Next point
				Tension,			// Tension
				false,				// Want clamping?
				ArriveTangent);		// Out

			// In 'auto' mode, arrive and leave tangents are always the same
			LeaveTangent = ArriveTangent;
		}

		Key.ArriveTangent = ArriveTangent;
		Key.LeaveTangent = LeaveTangent;
	}
}


void FRichCurve::ReadjustTimeRange(float NewMinTimeRange, float NewMaxTimeRange, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime)
{
	// first readjust modified time keys
	float ModifiedDuration = OldEndTime - OldStartTime;

	if (bInsert)
	{
		for(int32 KeyIndex=0; KeyIndex<Keys.Num(); ++KeyIndex)
		{
			float& CurrentTime = Keys[KeyIndex].Time;
			if (CurrentTime >= OldStartTime)
			{
				CurrentTime += ModifiedDuration;
			}
		}
	}
	else
	{
		// since we only allow one key at a given time, we will just cache the value that needs to be saved
		// this is the key to be replaced when this section is gone
		bool bAddNewKey = false; 
		float NewValue = 0.f;
		TArray<int32> KeysToDelete;

		for(int32 KeyIndex=0; KeyIndex<Keys.Num(); ++KeyIndex)
		{
			float& CurrentTime = Keys[KeyIndex].Time;
			// if this key exists between range of deleted
			// we'll evaluate the value at the "OldStartTime"
			// and re-add key, so that it keeps the previous value at the
			// start time
			// But that means if there are multiple keys, 
			// since we don't want multiple values in the same time
			// the last one will override the value
			if( CurrentTime >= OldStartTime && CurrentTime <= OldEndTime)
			{
				// get new value and add new key on one of OldStartTime, OldEndTime;
				// this is a bit complicated problem since we don't know if OldStartTime or OldEndTime is preferred. 
				// generall we use OldEndTime unless OldStartTime == 0.f
				// which means it's cut in the beginning. Otherwise it will always use the end time. 
				bAddNewKey = true;
				if (OldStartTime != 0.f)
				{
					NewValue = Eval(OldStartTime);
				}
				else
				{
					NewValue = Eval(OldEndTime);
				}
				// remove this key, but later because it might change eval result
				KeysToDelete.Add(KeyIndex);
			}
			else if (CurrentTime > OldEndTime)
			{
				CurrentTime -= ModifiedDuration;
			}
		}

		if (bAddNewKey)
		{
			for (int32 KeyIndex = KeysToDelete.Num()-1; KeyIndex >= 0; --KeyIndex)
			{
				const FKeyHandle KeyHandle = GetKeyHandle(KeysToDelete[KeyIndex]);
				if(IsKeyHandleValid(KeyHandle))
				{
					DeleteKey(KeyHandle);
				}
			}

			UpdateOrAddKey(OldStartTime, NewValue);
		}
	}

	RemoveRedundantAutoTangentKeys(0.f);

	// now cull out all out of range 
	float MinTime, MaxTime;
	GetTimeRange(MinTime, MaxTime);

	bool bNeedToDeleteKey=false;

	// if there is key below min time, just add key at new min range, 
	if (MinTime < NewMinTimeRange)
	{
		float NewValue = Eval(NewMinTimeRange);
		UpdateOrAddKey(NewMinTimeRange, NewValue);

		bNeedToDeleteKey = true;
	}

	// if there is key after max time, just add key at new max range, 
	if(MaxTime > NewMaxTimeRange)
	{
		float NewValue = Eval(NewMaxTimeRange);
		UpdateOrAddKey(NewMaxTimeRange, NewValue);

		bNeedToDeleteKey = true;
	}

	// delete the keys outside of range
	if (bNeedToDeleteKey)
	{
		// Ensure this here as DeleteKey might do it while running through the loop, causing the keyhandle to become invalid
		EnsureAllIndicesHaveHandles();
		for (int32 KeyIndex=0; KeyIndex<Keys.Num(); ++KeyIndex)
		{
			if (Keys[KeyIndex].Time < NewMinTimeRange || Keys[KeyIndex].Time > NewMaxTimeRange)
			{
				const FKeyHandle* KeyHandle = KeyHandlesToIndices.FindKey(KeyIndex);
				if (KeyHandle && IsKeyHandleValid(*KeyHandle))
				{
					DeleteKey(*KeyHandle);
					--KeyIndex;
				}
			}
		}
	}
}

void FRichCurve::BakeCurve(float SampleRate)
{
	if (Keys.Num() == 0)
	{
		return;
	}

	float FirstKeyTime = Keys[0].Time;
	float LastKeyTime = Keys[Keys.Num()-1].Time;

	BakeCurve(SampleRate, FirstKeyTime, LastKeyTime);
}

void FRichCurve::BakeCurve(float SampleRate, float FirstKeyTime, float LastKeyTime)
{
	if (Keys.Num() == 0)
	{
		return;
	}

	// we need to generate new keys first rather than modifying the
	// curve directly since that would affect the results of Eval calls
	TArray<TPair<float, float> > BakedKeys;
	BakedKeys.Reserve(((LastKeyTime - FirstKeyTime) / SampleRate) - 1);

	// the skip the first and last key unchanged
	for (float Time = FirstKeyTime + SampleRate; Time < LastKeyTime; )
	{
		const float Value = Eval(Time);
		BakedKeys.Add(TPair<float, float>(Time, Value));
		Time += SampleRate;
	}

	for (const TPair<float,float>& NewKey : BakedKeys)
	{
		UpdateOrAddKey(NewKey.Key, NewKey.Value);
	}
}

void FRichCurve::RemoveRedundantAutoTangentKeys(float Tolerance)
{
	RemoveRedundantKeys(Tolerance, FFrameRate(0,0));
}

void FRichCurve::RemoveRedundantKeys(float Tolerance, FFrameRate SampleRate)
{
	if (Keys.Num() < 3)
	{
		return;
	}

	RemoveRedundantKeysInternal(Tolerance, 0, Keys.Num() - 1, SampleRate);
}

void FRichCurve::RemoveRedundantAutoTangentKeys(float Tolerance, float FirstKeyTime, float LastKeyTime)
{
	RemoveRedundantKeys(Tolerance, FirstKeyTime, LastKeyTime, FFrameRate(0,0));
}

void FRichCurve::RemoveRedundantKeys(float Tolerance, float FirstKeyTime, float LastKeyTime, FFrameRate SampleRate)
{
	if (FirstKeyTime >= LastKeyTime)
	{
		return;
	}
	int32 StartKey = -1;
	int32 EndKey = -1;
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		const float CurrentKeyTime = Keys[KeyIndex].Time;

		if (CurrentKeyTime <= FirstKeyTime)
		{
			StartKey = KeyIndex;
		}
		if (CurrentKeyTime >= LastKeyTime)
		{
			EndKey = KeyIndex;
			break;
		}
	}

	if ((StartKey != INDEX_NONE) && (EndKey != INDEX_NONE))
	{
		RemoveRedundantKeysInternal(Tolerance, StartKey, EndKey, SampleRate);
	}
}


void FRichCurve::RemoveRedundantKeysInternal(float Tolerance, int32 InStartKeepKey, int32 InEndKeepKey, FFrameRate SampleRate)
{
	if (Keys.Num() < 3) // Will always keep first and last key
	{
		return;
	}

	const int32 ActualStartKeepKey = FMath::Max(InStartKeepKey, 0); // Will always keep first and last key
	const int32 ActualEndKeepKey = FMath::Min(InEndKeepKey, Keys.Num()-1);

	check(ActualStartKeepKey < ActualEndKeepKey); // Make sure we are doing something sane
	if ((ActualEndKeepKey - ActualStartKeepKey) < 2)
	{
		//Not going to do anything useful
		return;
	}

	//Build some helper data for managing the HandleTokey map
	TArray<FKeyHandle> AllHandlesByIndex;
	TArray<FKeyHandle> KeepHandles;

	// If we have no handles to start with do not bother generating any
	const bool bHasKeyHandles = KeyHandlesToIndices.Num() != 0;
	if (bHasKeyHandles)
	{
		check(KeyHandlesToIndices.Num() == Keys.Num());
		AllHandlesByIndex.AddZeroed(Keys.Num());
		KeepHandles.Reserve(Keys.Num());

		for (const TPair<FKeyHandle, int32>& HandleIndexPair : KeyHandlesToIndices.GetMap())
		{
			AllHandlesByIndex[HandleIndexPair.Value] = HandleIndexPair.Key;
		}
	}

	{
		TArray<FRichCurveKey> NewKeys;
		NewKeys.Reserve(Keys.Num());

		//Add all the keys we are keeping from the start
		for(int32 StartKeepIndex = 0; StartKeepIndex <= ActualStartKeepKey; ++StartKeepIndex)
		{
			NewKeys.Add(Keys[StartKeepIndex]);
			
			if (bHasKeyHandles)
			{
				KeepHandles.Add(AllHandlesByIndex[StartKeepIndex]);
			}
		}

		//Add keys up to the first end keep key if they are not redundant
		int32 MostRecentKeepKeyIndex = 0;

		const bool bTryRemoveUserTangentKeys = SampleRate.IsValid();
		for (int32 TestIndex = ActualStartKeepKey+1; TestIndex < ActualEndKeepKey; ++TestIndex) //Loop within the bounds of the first and last key
		{
			bool bKeepKey = false;
			
			if (bTryRemoveUserTangentKeys)
			{
				const FFrameTime KeyOneFrameTime = SampleRate.AsFrameTime(Keys[MostRecentKeepKeyIndex].Time);
				const FFrameTime KeyTwoFrameTime = SampleRate.AsFrameTime(Keys[TestIndex + 1].Time);

				FFrameTime SampleFrameTime = KeyOneFrameTime;
				const float ToRemoveKeyTime = Keys[TestIndex].Time;

				// Sample curve at intervals, dictated by provided sample-rate, between the keys surrounding the to-be-removed key
				// this makes sure that any impact on non-lerp keys is taken into account
				while (SampleFrameTime <= KeyTwoFrameTime)
				{
					const float SampleSeconds = SampleRate.AsSeconds(SampleFrameTime);
					const float ValueWithoutKey = UE::Curves::EvalForTwoKeys(Keys[MostRecentKeepKeyIndex], Keys[TestIndex + 1], SampleSeconds);

					const FRichCurveKey& KeyOne = SampleSeconds <= ToRemoveKeyTime ? Keys[MostRecentKeepKeyIndex] : Keys[TestIndex];
					const FRichCurveKey& KeyTwo = SampleSeconds <= ToRemoveKeyTime ? Keys[TestIndex] : Keys[TestIndex + 1];
					const float ValueWithKey = UE::Curves::EvalForTwoKeys(KeyOne, KeyTwo, SampleSeconds);
				
					if (FMath::Abs(ValueWithoutKey - ValueWithKey) > Tolerance)
					{
						bKeepKey = true;
						break;
					}

					// Next frame
					SampleFrameTime.FrameNumber += 1;	
				}
			}
			else
			{
				auto IsNonUserTangentKey = [](const FRichCurveKey& Key)
				{
					return Key.TangentMode != RCTM_User && Key.TangentMode != RCTM_Break;
				};

				bKeepKey = true;
				if (IsNonUserTangentKey(Keys[MostRecentKeepKeyIndex]) && IsNonUserTangentKey(Keys[TestIndex + 1]) && IsNonUserTangentKey(Keys[TestIndex]))
				{
					const float KeyValue = Keys[TestIndex].Value;
					const float ValueWithoutKey = UE::Curves::EvalForTwoKeys(Keys[MostRecentKeepKeyIndex], Keys[TestIndex + 1], Keys[TestIndex].Time);

					bKeepKey = FMath::Abs(ValueWithoutKey - KeyValue) > Tolerance;
				}
			}			
			
			if (bKeepKey) // Is this key needed
			{
				MostRecentKeepKeyIndex = TestIndex;
				NewKeys.Add(Keys[TestIndex]);

				if(bHasKeyHandles)
				{
					KeepHandles.Add(AllHandlesByIndex[TestIndex]);
				}
			}
		}

		//Add end keys that we are keeping
		for (int32 EndKeepIndex = ActualEndKeepKey; EndKeepIndex < Keys.Num(); ++EndKeepIndex)
		{
			NewKeys.Add(Keys[EndKeepIndex]);
			if (bHasKeyHandles)
			{
				KeepHandles.Add(AllHandlesByIndex[EndKeepIndex]);
			}
		}
		Keys = MoveTemp(NewKeys); //Do this at the end of scope, guaranteed that NewKeys is going away
	}

	AutoSetTangents();

	if (bHasKeyHandles)
	{
		// Rebuild KeyHandlesToIndices
		check(Keys.Num() == KeepHandles.Num());
		KeyHandlesToIndices.Initialize(KeepHandles);
	}
}

void FRichCurve::RemapTimeValue(float& InTime, float& CycleValueOffset) const
{
	const int32 NumKeys = Keys.Num();
	
	if (NumKeys < 2)
	{
		return;
	} 

	if (InTime <= Keys[0].Time)
	{
		if (PreInfinityExtrap != RCCE_Linear && PreInfinityExtrap != RCCE_Constant)
		{
			float MinTime = Keys[0].Time;
			float MaxTime = Keys[NumKeys - 1].Time;

			int CycleCount = 0;
			CycleTime(MinTime, MaxTime, InTime, CycleCount);

			if (PreInfinityExtrap == RCCE_CycleWithOffset)
			{
				float DV = Keys[0].Value - Keys[NumKeys - 1].Value;
				CycleValueOffset = DV * CycleCount;
			}
			else if (PreInfinityExtrap == RCCE_Oscillate)
			{
				if (CycleCount % 2 == 1)
				{
					InTime = MinTime + (MaxTime - InTime);
				}
			}
		}
	}
	else if (InTime >= Keys[NumKeys - 1].Time)
	{
		if (PostInfinityExtrap != RCCE_Linear && PostInfinityExtrap != RCCE_Constant)
		{
			float MinTime = Keys[0].Time;
			float MaxTime = Keys[NumKeys - 1].Time;

			int CycleCount = 0; 
			CycleTime(MinTime, MaxTime, InTime, CycleCount);

			if (PostInfinityExtrap == RCCE_CycleWithOffset)
			{
				float DV = Keys[NumKeys - 1].Value - Keys[0].Value;
				CycleValueOffset = DV * CycleCount;
			}
			else if (PostInfinityExtrap == RCCE_Oscillate)
			{
				if (CycleCount % 2 == 1)
				{
					InTime = MinTime + (MaxTime - InTime);
				}
			}
		}
	}
}

float FRichCurve::Eval(float InTime, float InDefaultValue) const
{
	// Remap time if extrapolation is present and compute offset value to use if cycling 
	float CycleValueOffset = 0;
	RemapTimeValue(InTime, CycleValueOffset);

	const int32 NumKeys = Keys.Num();

	// If the default value hasn't been initialized, use the incoming default value
	float InterpVal = DefaultValue == MAX_flt ? InDefaultValue : DefaultValue;

	if (NumKeys == 0)
	{
		// If no keys in curve, return the Default value.
	} 
	else if (NumKeys < 2 || (InTime < Keys[0].Time))
	{
		if (PreInfinityExtrap == RCCE_Linear && NumKeys > 1)
		{
			float DT = Keys[1].Time - Keys[0].Time;
			
			if (FMath::IsNearlyZero(DT))
			{
				InterpVal = Keys[0].Value;
			}
			else
			{
				float DV = Keys[1].Value - Keys[0].Value;
				float Slope = DV / DT;

				InterpVal = Slope * (InTime - Keys[0].Time) + Keys[0].Value;
			}
		}
		else
		{
			// Otherwise if constant or in a cycle or oscillate, always use the first key value
			InterpVal = Keys[0].Value;
		}
	}
	else if (InTime <= Keys[NumKeys - 1].Time)
	{
		// perform a lower bound to get the second of the interpolation nodes
		int32 first = 1;
		int32 last = NumKeys - 1;
		int32 count = last - first;

		while (count > 0)
		{
			const int32 step = count / 2;
			const int32 middle = first + step;

			if (InTime >= Keys[middle].Time)
			{
				first = middle + 1;
				count -= step + 1;
			}
			else
			{
				count = step;
			}
		}

		InterpVal = UE::Curves::EvalForTwoKeys(Keys[first - 1], Keys[first], InTime);
	}
	else
	{
		if (PostInfinityExtrap == RCCE_Linear)
		{
			float DT = Keys[NumKeys - 2].Time - Keys[NumKeys - 1].Time;
			
			if (FMath::IsNearlyZero(DT))
			{
				InterpVal = Keys[NumKeys - 1].Value;
			}
			else
			{
				float DV = Keys[NumKeys - 2].Value - Keys[NumKeys - 1].Value;
				float Slope = DV / DT;

				InterpVal = Slope * (InTime - Keys[NumKeys - 1].Time) + Keys[NumKeys - 1].Value;
			}
		}
		else
		{
			// Otherwise if constant or in a cycle or oscillate, always use the last key value
			InterpVal = Keys[NumKeys - 1].Value;
		}
	}

	return InterpVal+CycleValueOffset;
}


bool FRichCurve::operator==(const FRichCurve& Curve) const
{
	if(Keys.Num() != Curve.Keys.Num())
	{
		return false;
	}

	for(int32 i = 0;i<Keys.Num();++i)
	{
		if(!(Keys[i] == Curve.Keys[i]))
		{
			return false;
		}
	}

	if (PreInfinityExtrap != Curve.PreInfinityExtrap || PostInfinityExtrap != Curve.PostInfinityExtrap)
	{
		return false;
	}

	return true;
}

static ERichCurveCompressionFormat FindRichCurveCompressionFormat(const FRichCurve& Curve)
{
	if (Curve.Keys.Num() == 0)
	{
		return RCCF_Empty;
	}

	if (Curve.IsConstant())
	{
		return RCCF_Constant;
	}

	const FRichCurveKey& RefKey = Curve.Keys[0];
	bool bIsMixed = false;
	bool bIsWeighted = false;
	for (const FRichCurveKey& Key : Curve.Keys)
	{
		if (Key.InterpMode == RCIM_Cubic)
		{
			if (Key.TangentWeightMode != RCTWM_WeightedNone)
			{
				bIsWeighted = true;
				break;
			}
		}
		if (Key.InterpMode != RefKey.InterpMode)
		{
			bIsMixed = true;
		}
		//RefKey = Key; 
	}
	if (bIsWeighted)
	{
		return RCCF_Weighted;
	}
	else if(bIsMixed)
	{
		return RCCF_Mixed;
	}
	switch (RefKey.InterpMode)
	{
	case RCIM_Constant:
		return RCCF_Mixed;
	case RCIM_None:
	default:
		return RCCF_Constant;
	case RCIM_Linear:
		return RCCF_Linear;
	case RCIM_Cubic:
		return RCCF_Cubic;
	}
}

static ERichCurveKeyTimeCompressionFormat FindRichCurveKeyFormat(const FRichCurve& Curve, float ErrorThreshold, float SampleRate, ERichCurveCompressionFormat CompressionFormat)
{
	const int32 NumKeys = Curve.Keys.Num();
	if (NumKeys == 0 || CompressionFormat == RCCF_Constant || CompressionFormat == RCCF_Empty || ErrorThreshold <= 0.0f || SampleRate <= 0.0f)
	{
		return RCKTCF_float32;
	}

	auto EvalForTwoKeys = [](const FRichCurveKey& Key1, float KeyTime1, const FRichCurveKey& Key2, float KeyTime2, float InTime)
	{
		const float Diff = KeyTime2 - KeyTime1;

		if (Diff > 0.f && Key1.InterpMode != RCIM_Constant)
		{
			const float Alpha = (InTime - KeyTime1) / Diff;
			const float P0 = Key1.Value;
			const float P3 = Key2.Value;

			if (Key1.InterpMode == RCIM_Linear)
			{
				return FMath::Lerp(P0, P3, Alpha);
			}
			else if(UE::Curves::IsItNotWeighted(Key1,Key2))
			{
				const float OneThird = 1.0f / 3.0f;
				const float P1 = P0 + (Key1.LeaveTangent * Diff * OneThird);
				const float P2 = P3 - (Key2.ArriveTangent * Diff * OneThird);

				return UE::Curves::BezierInterp(P0, P1, P2, P3, Alpha);
			}
			else //it's weighted
			{
				return  UE::Curves::WeightedEvalForTwoKeys(
					Key1.Value, KeyTime1, Key1.LeaveTangent, Key1.LeaveTangentWeight, Key1.TangentWeightMode,
					Key2.Value, KeyTime2, Key2.ArriveTangent, Key2.ArriveTangentWeight, Key2.TangentWeightMode,
					InTime);
			}
		}
		else
		{
			return Key1.Value;
		}
	};

	auto DecayTime = [](const FRichCurveKey& Key, float MinTime, float DeltaTime, float InvDeltaTime, float QuantizationScale, float InvQuantizationScale)
	{
		// 0.0f -> 0, 1.0f -> 255 for 8 bits
		const float NormalizedTime = FMath::Clamp((Key.Time - MinTime) * InvDeltaTime, 0.0f, 1.0f);
		const float QuantizedTime = FMath::RoundHalfFromZero(NormalizedTime * QuantizationScale);
		const float LossyNormalizedTime = QuantizedTime * InvQuantizationScale;
		return (LossyNormalizedTime * DeltaTime) + MinTime;
	};

	const float MinTime = Curve.Keys[0].Time;
	const float MaxTime = Curve.Keys.Last().Time;
	const float DeltaTime = MaxTime - MinTime;
	const float InvDeltaTime = 1.0f / DeltaTime;
	const float SampleRateIncrement = 1.0f / SampleRate;

	// This is only acceptable if the maximum error is within a reasonable value
	bool bFitsOn16Bits = true;

	int32 CurrentLossyKey = 0;
	int32 CurrentRefKey = 0;
	float CurrentTime = MinTime;
	while (CurrentTime <= MaxTime && bFitsOn16Bits)
	{
		if (CurrentTime > Curve.Keys[CurrentRefKey + 1].Time)
		{
			CurrentRefKey++;

			if (CurrentRefKey >= NumKeys)
			{
				break;
			}
		}

		float LossyTime1_16;
		float LossyTime2_16 = DecayTime(Curve.Keys[CurrentLossyKey + 1], MinTime, DeltaTime, InvDeltaTime, 65535.0f, 1.0f / 65535.0f);
		if (CurrentTime > LossyTime2_16)
		{
			CurrentLossyKey++;

			if (CurrentLossyKey >= NumKeys)
			{
				break;
			}

			LossyTime1_16 = LossyTime2_16;
			LossyTime2_16 = DecayTime(Curve.Keys[CurrentLossyKey + 1], MinTime, DeltaTime, InvDeltaTime, 65535.0f, 1.0f / 65535.0f);
		}
		else
		{
			LossyTime1_16 = DecayTime(Curve.Keys[CurrentLossyKey], MinTime, DeltaTime, InvDeltaTime, 65535.0f, 1.0f / 65535.0f);
		}

		const float Result_16 = EvalForTwoKeys(Curve.Keys[CurrentLossyKey], LossyTime1_16, Curve.Keys[CurrentLossyKey + 1], LossyTime2_16, CurrentTime);
		const float Result_Ref = UE::Curves::EvalForTwoKeys(Curve.Keys[CurrentRefKey], Curve.Keys[CurrentRefKey + 1], CurrentTime);

		const float Error_16 = FMath::Abs(Result_Ref - Result_16);

		bFitsOn16Bits &= Error_16 <= ErrorThreshold;

		CurrentTime += SampleRateIncrement;
	}

	// In order to normalize time values, we need to store the MinTime and the DeltaTime with full precision
	// This means we need 8 bytes of overhead
	// If the number of keys is too small, the overhead is larger or equal to the space we save and isn't
	// worth it.

	// For 8 bits, the formula is: 8 + (N * sizeof(uint8))
	// With 8 bits, 2 keys need 8 bytes with full precision and 10 bytes packed. No savings, not worth using.
	// With 8 bits, 3 keys need 12 bytes with full precision and 11 bytes packed. We save 1 byte, worth using.
	// For 16 bits, the formula is: 8 + (N * sizeof(uint16))
	// With 16 bits, 6 keys need 20 bytes with full precision and 20 bytes packed. No savings, not worth using.
	// With 16 bits, 7 keys need 24 bytes with full precision and 22 bytes packed. We save 2 bytes, worth using.
	// Alignment and padding must also be taken into account

	// Note: Support for storing key time on 8 bits was attempted but it was rarely selected and wasn't worth the complexity

	int32 SizeInterpMode = 0;
	if (CompressionFormat == RCCF_Mixed)
	{
		SizeInterpMode += NumKeys * sizeof(uint8);
	}
	else if (CompressionFormat == RCCF_Weighted)
	{
		SizeInterpMode += NumKeys * 2 * sizeof(uint8);

	}

	const int32 SizeUInt16 = Align(Align(SizeInterpMode, sizeof(uint16)) + (NumKeys * sizeof(uint16)), sizeof(float)) + (2 * sizeof(float));
	const int32 SizeFloat32 = Align(SizeInterpMode, sizeof(float)) + NumKeys * sizeof(float);

	if (bFitsOn16Bits && SizeUInt16 < SizeFloat32)
	{
		return RCKTCF_uint16;
	}
	else
	{
		return RCKTCF_float32;
	}
}

void FRichCurve::CompressCurve(FCompressedRichCurve& OutCurve, float ErrorThreshold, float SampleRate) const
{
	const ERichCurveCompressionFormat CompressionFormat = FindRichCurveCompressionFormat(*this);
	OutCurve.CompressionFormat = CompressionFormat;

	const ERichCurveKeyTimeCompressionFormat KeyFormat = FindRichCurveKeyFormat(*this, ErrorThreshold, SampleRate, CompressionFormat);
	OutCurve.KeyTimeCompressionFormat = KeyFormat;

	OutCurve.PreInfinityExtrap = PreInfinityExtrap;
	OutCurve.PostInfinityExtrap = PostInfinityExtrap;

	if (CompressionFormat == RCCF_Empty)
	{
		OutCurve.ConstantValueNumKeys.ConstantValue = DefaultValue;
		OutCurve.CompressedKeys.Empty();
	}
	else if (CompressionFormat == RCCF_Constant)
	{
		OutCurve.ConstantValueNumKeys.ConstantValue = Keys[0].Value;
		OutCurve.CompressedKeys.Empty();
	}
	else
	{
		int32 PackedDataSize = 0;

		// If we are mixed, we need to store the interp mode for every key, this data comes first following the header
		// Next comes the quantized time values followed by the normalization range
		// And the values/tangents follow last

		if (CompressionFormat == RCCF_Mixed)
		{
			PackedDataSize += Keys.Num() * sizeof(uint8);
		}
		else if (CompressionFormat == RCCF_Weighted)
		{
			//Weighted has interp mode and tangent weight mode
			PackedDataSize += Keys.Num() * 2 * sizeof(uint8);
		}

		if (KeyFormat == RCKTCF_uint16)
		{
			PackedDataSize = Align(PackedDataSize, sizeof(uint16));
			PackedDataSize += Keys.Num() * sizeof(uint16);
			PackedDataSize = Align(PackedDataSize, sizeof(float));
			PackedDataSize += 2 * sizeof(float);
		}
		else
		{
			check(KeyFormat == RCKTCF_float32);
			PackedDataSize = Align(PackedDataSize, sizeof(float));
			PackedDataSize += Keys.Num() * sizeof(float);
		}

		PackedDataSize += Keys.Num() * sizeof(float);	// Key values

		// Key tangents
		if (CompressionFormat == RCCF_Cubic)
		{
			PackedDataSize += Keys.Num() * 2 * sizeof(float);
		}
		else if (CompressionFormat == RCCF_Mixed)
		{
#if MIXEDKEY_STRIPS_TANGENTS
			for (const FRichCurveKey& Key : Keys)
			{
				if (Key.InterpMode == RCIM_Cubic)
				{
					PackedDataSize += 2 * sizeof(float);
				}
			}
#else
			PackedDataSize += Keys.Num() * 2 * sizeof(float); // Always have tangents
#endif
		}
		else if (CompressionFormat == RCCF_Weighted)
		{
			//Weighted has 2 tangents and 2 tangent weights
			PackedDataSize += Keys.Num() * 4 * sizeof(float);
		}

		OutCurve.CompressedKeys.Empty(PackedDataSize);
		// Zero-initialize rather than AddUninitialized: the interp-mode/time/value sections below are
		// written at Align()'d offsets, so the alignment-gap bytes are never written. Leaving them
		// uninitialized leaks heap garbage into the serialized payload and makes the AnimationSequence
		// DDC output non-deterministic from run to run (DDC-Verify mismatches). Zeroing keeps the gaps stable.
		OutCurve.CompressedKeys.AddZeroed(PackedDataSize);

		int32 WriteOffset = 0;
		uint8* BasePtr = OutCurve.CompressedKeys.GetData();

		OutCurve.ConstantValueNumKeys.NumKeys = Keys.Num();

		// Key interp modes
		if (CompressionFormat == RCCF_Mixed)
		{
			uint8* InterpModes = BasePtr + WriteOffset;
			WriteOffset += Keys.Num() * sizeof(uint8);

			for (const FRichCurveKey& Key : Keys)
			{
				if (Key.InterpMode == RCIM_Linear)
				{
					*InterpModes++ = (uint8)RCCF_Linear;
				}
				else if (Key.InterpMode == RCIM_Cubic)
				{
					*InterpModes++ = (uint8)RCCF_Cubic;
				}
				else
				{
					*InterpModes++ = (uint8)RCCF_Constant;
				}
			}
		}
		else if (CompressionFormat == RCCF_Weighted)
		{
			uint8* InterpModes = BasePtr + WriteOffset;
			WriteOffset += Keys.Num() *  2 * sizeof(uint8);

			for (const FRichCurveKey& Key : Keys)
			{
				if (Key.InterpMode == RCIM_Linear)
				{
					*InterpModes++ = (uint8)RCCF_Linear;
				}
				else if (Key.InterpMode == RCIM_Cubic)
				{
					*InterpModes++ = (uint8)RCCF_Cubic;
				}
				else
				{
					*InterpModes++ = (uint8)RCCF_Constant;
				}

				*InterpModes++ = (uint8)Key.TangentWeightMode;

			}
		}

		// Key times
		if (KeyFormat == RCKTCF_uint16)
		{
			const float MinTime = Keys[0].Time;
			const float MaxTime = Keys.Last().Time;
			const float DeltaTime = MaxTime - MinTime;
			const float InvDeltaTime = 1.0f / DeltaTime;

			const int32 KeySize = sizeof(uint16);

			WriteOffset = Align(WriteOffset, sizeof(uint16));

			uint8* KeyTimes8 = BasePtr + WriteOffset;
			uint16* KeyTimes16 = reinterpret_cast<uint16*>(KeyTimes8);
			WriteOffset += Keys.Num() * KeySize;

			for (const FRichCurveKey& Key : Keys)
			{
				const float NormalizedTime = FMath::Clamp((Key.Time - MinTime) * InvDeltaTime, 0.0f, 1.0f);
				const uint16 QuantizedTime = (uint16)FMath::RoundHalfFromZero(NormalizedTime * 65535.0f);
				*KeyTimes16++ = QuantizedTime;
			}

			WriteOffset = Align(WriteOffset, sizeof(float));
			float* RangeData = reinterpret_cast<float*>(BasePtr + WriteOffset);
			WriteOffset += 2 * sizeof(float);

			RangeData[0] = MinTime;
			RangeData[1] = DeltaTime;
		}
		else
		{
			WriteOffset = Align(WriteOffset, sizeof(float));
			float* KeyTimes = reinterpret_cast<float*>(BasePtr + WriteOffset);
			WriteOffset += Keys.Num() * sizeof(float);

			for (const FRichCurveKey& Key : Keys)
			{
				*KeyTimes++ = Key.Time;
			}
		}

		// Key values and tangents
		float* KeyData = reinterpret_cast<float*>(BasePtr + WriteOffset);
		for (const FRichCurveKey& Key : Keys)
		{
			*KeyData++ = Key.Value;

#if MIXEDKEY_STRIPS_TANGENTS
			if (Key.InterpMode == RCIM_Cubic)
#else
			if (CompressionFormat == RCCF_Mixed || Key.InterpMode == RCIM_Cubic || CompressionFormat == RCCF_Weighted)
#endif
			{
				check(CompressionFormat == RCCF_Cubic || CompressionFormat == RCCF_Mixed|| CompressionFormat == RCCF_Weighted);
				*KeyData++ = Key.ArriveTangent;
				*KeyData++ = Key.LeaveTangent;
			}
			if (CompressionFormat == RCCF_Weighted)
			{
				*KeyData++ = Key.ArriveTangentWeight;
				*KeyData++ = Key.LeaveTangentWeight;
			}
		}
		check(((uint8*)KeyData - BasePtr) == PackedDataSize);
	}
}

void FRichCurve::ConvertToChaosCurve(Chaos::FLinearCurve& OutCurve) const
{
	OutCurve.Keys.Reset();

	for (const FRichCurveKey& Key: Keys)
	{
		OutCurve.Keys.Add({ Key.Time, Key.Value });
	}
}

struct Quantized16BitKeyTimeAdapter
{
	static constexpr float QuantizationScale = 1.0f / 65535.0f;
	static constexpr int32 KeySize = sizeof(uint16);
	static constexpr int32 RangeDataSize = 2 * sizeof(float);

	using KeyTimeType = uint16;

	const uint16* KeyTimes;
	float MinTime;
	float DeltaTime;
	int32 KeyDataOffset;

	Quantized16BitKeyTimeAdapter(const uint8* BasePtr, int32 KeyTimesOffset, int32 NumKeys)
	{
		const int32 RangeDataOffset = Align(KeyTimesOffset + (NumKeys * sizeof(uint16)), sizeof(float));
		KeyDataOffset = RangeDataOffset + RangeDataSize;
		const float* RangeData = reinterpret_cast<const float*>(BasePtr + RangeDataOffset);

		KeyTimes = reinterpret_cast<const uint16*>(BasePtr + KeyTimesOffset);
		MinTime = RangeData[0];
		DeltaTime = RangeData[1];
	}

	float GetTime(int32 KeyIndex) const
	{
		const float KeyNormalizedTime = KeyTimes[KeyIndex] * QuantizationScale;
		return (KeyNormalizedTime * DeltaTime) + MinTime;
	};
};

struct Float32BitKeyTimeAdapter
{
	static constexpr int32 KeySize = sizeof(float);
	static constexpr int32 RangeDataSize = 0;

	using KeyTimeType = float;

	const float* KeyTimes;
	int32 KeyDataOffset;

	Float32BitKeyTimeAdapter(const uint8* BasePtr, int32 KeyTimesOffset, int32 NumKeys)
	{
		KeyTimes = reinterpret_cast<const float*>(BasePtr + KeyTimesOffset);
		KeyDataOffset = Align(KeyTimesOffset + (NumKeys * sizeof(float)), sizeof(float));
	}

	constexpr float GetTime(int32 KeyIndex) const
	{
		return KeyTimes[KeyIndex];
	};
};

using KeyDataHandle = int32;

template<ERichCurveCompressionFormat Format>
struct UniformKeyDataAdapter
{
	const float* KeyData;

	template<typename KeyTimeAdapterType>
	
	constexpr UniformKeyDataAdapter(const uint8* BasePtr, const KeyTimeAdapterType& KeyTimeAdapter)
    : KeyData(reinterpret_cast<const float*>(BasePtr + KeyTimeAdapter.KeyDataOffset))
    {}

	constexpr KeyDataHandle GetKeyDataHandle(int32 KeyIndexToQuery) const
	{
		return Format == RCCF_Cubic ? (KeyIndexToQuery * 3) : KeyIndexToQuery;
	};

	constexpr float GetKeyValue(KeyDataHandle Handle) const
	{
		return KeyData[Handle];
	}

	constexpr float GetKeyArriveTangent(KeyDataHandle Handle) const
	{
		return KeyData[Handle + 1];
	}

	constexpr float GetKeyLeaveTangent(KeyDataHandle Handle) const
	{
		return KeyData[Handle + 2];
	}

	constexpr ERichCurveCompressionFormat GetKeyInterpMode(int32 KeyIndex) const
	{
		return Format;
	}

	constexpr ERichCurveTangentWeightMode GetKeyTangentWeightMode(int32 KeyIndex) const
	{
		return RCTWM_WeightedNone;
	}

	constexpr float GetKeyArriveTangentWeight(KeyDataHandle Handle) const
	{
		return 0.0f;
	}

	constexpr float GetKeyLeaveTangentWeight(KeyDataHandle Handle) const
	{
		return 0.0f;
	}
	
};

struct MixedKeyDataAdapter
{
	const uint8* InterpModes;
	const float* KeyData;

	template<typename KeyTimeAdapterType>
	MixedKeyDataAdapter(const uint8* BasePtr, int32 InterpModesOffset, const KeyTimeAdapterType& KeyTimeAdapter)
	{
		InterpModes = BasePtr + InterpModesOffset;
		KeyData = reinterpret_cast<const float*>(BasePtr + KeyTimeAdapter.KeyDataOffset);
	}

	KeyDataHandle GetKeyDataHandle(int32 KeyIndexToQuery) const
	{
#if MIXEDKEY_STRIPS_TANGENTS
		int32 Offset = 0;
		for (int32 KeyIndex = 0; KeyIndex < KeyIndexToQuery; ++KeyIndex)
		{
			Offset += InterpModes[KeyIndex] == RCCF_Cubic ? 3 : 1;
		}

		return Offset;
#else
		return KeyIndexToQuery * 3;
#endif
	};

	constexpr float GetKeyValue(KeyDataHandle Handle) const
	{
		return KeyData[Handle];
	}

	constexpr float GetKeyArriveTangent(KeyDataHandle Handle) const
	{
		return KeyData[Handle + 1];
	}

	constexpr float GetKeyLeaveTangent(KeyDataHandle Handle) const
	{
		return KeyData[Handle + 2];
	}

	constexpr ERichCurveCompressionFormat GetKeyInterpMode(int32 KeyIndex) const
	{
		return (ERichCurveCompressionFormat)InterpModes[KeyIndex];
	}

	constexpr ERichCurveTangentWeightMode GetKeyTangentWeightMode(int32 KeyIndex) const
	{
		return RCTWM_WeightedNone;
	}

	constexpr float GetKeyArriveTangentWeight(KeyDataHandle Handle) const
	{
		return 0.0f;
	}

	constexpr float GetKeyLeaveTangentWeight(KeyDataHandle Handle) const
	{
		return 0.0f;
	}
};


struct WeightedKeyDataAdapter
{
	const uint8* InterpModes;
	const float* KeyData;

	template<typename KeyTimeAdapterType>
	WeightedKeyDataAdapter(const uint8* BasePtr, int32 InterpModesOffset, const KeyTimeAdapterType& KeyTimeAdapter)
	{
		InterpModes = BasePtr + InterpModesOffset;
		KeyData = reinterpret_cast<const float*>(BasePtr + KeyTimeAdapter.KeyDataOffset);
	}

	KeyDataHandle GetKeyDataHandle(int32 KeyIndexToQuery) const
	{
		return KeyIndexToQuery * 5;
	};

	constexpr float GetKeyValue(KeyDataHandle Handle) const
	{
		return KeyData[Handle];
	}

	constexpr float GetKeyArriveTangent(KeyDataHandle Handle) const
	{
		return KeyData[Handle + 1];
	}

	constexpr float GetKeyLeaveTangent(KeyDataHandle Handle) const
	{
		return KeyData[Handle + 2];
	}

	constexpr float GetKeyArriveTangentWeight(KeyDataHandle Handle) const
	{
		return KeyData[Handle + 3];
	}

	constexpr float GetKeyLeaveTangentWeight(KeyDataHandle Handle) const
	{
		return KeyData[Handle + 4];
	}

	constexpr ERichCurveCompressionFormat GetKeyInterpMode(int32 KeyIndex) const
	{
		return (ERichCurveCompressionFormat)InterpModes[(KeyIndex * 2)];
	}

	constexpr ERichCurveTangentWeightMode GetKeyTangentWeightMode(int32 KeyIndex) const
	{
		return (ERichCurveTangentWeightMode)InterpModes[(KeyIndex * 2)+1];
	}


};
static void CycleTime(float MinTime, float MaxTime, float& InTime, int& CycleCount)
{
	float InitTime = InTime;
	float Duration = MaxTime - MinTime;

	if (InTime > MaxTime)
	{
		CycleCount = FMath::FloorToInt((MaxTime-InTime)/Duration);
		InTime = InTime + Duration*CycleCount;
	}
	else if (InTime < MinTime)
	{
		CycleCount = FMath::FloorToInt((InTime-MinTime)/Duration);
		InTime = InTime - Duration*CycleCount;
	}

	if (InTime == MaxTime && InitTime < MinTime)
	{
		InTime = MinTime;
	}

	if (InTime == MinTime && InitTime > MaxTime)
	{
		InTime = MaxTime;
	}

	CycleCount = FMath::Abs(CycleCount);
}

template<typename KeyTimeAdapterType, typename KeyDataAdapterType>
static float RemapTimeValue(float InTime, const KeyTimeAdapterType& KeyTimeAdapter, const KeyDataAdapterType& KeyDataAdapter, int32 NumKeys, ERichCurveExtrapolation InfinityExtrap, int32 KeyIndex0, int32 KeyIndex1, float& CycleValueOffset)
{
	// For Pre-infinity, key0 and key1 are the actual key 0 and key 1
	// For Post-infinity, key0 and key1 are the last and second to last key
	const float MinTime = KeyTimeAdapter.GetTime(0);
	const float MaxTime = KeyTimeAdapter.GetTime(NumKeys - 1);

	int CycleCount = 0;
	CycleTime(MinTime, MaxTime, InTime, CycleCount);

	if (InfinityExtrap == RCCE_CycleWithOffset)
	{
		const KeyDataHandle ValueHandle0 = KeyDataAdapter.GetKeyDataHandle(KeyIndex0);
		const float KeyValue0 = KeyDataAdapter.GetKeyValue(ValueHandle0);
		const KeyDataHandle ValueHandle1 = KeyDataAdapter.GetKeyDataHandle(KeyIndex1);
		const float KeyValue1 = KeyDataAdapter.GetKeyValue(ValueHandle1);

		const float DV = KeyValue0 - KeyValue1;
		CycleValueOffset = DV * CycleCount;
	}
	else if (InfinityExtrap == RCCE_Oscillate)
	{
		if (CycleCount % 2 == 1)
		{
			InTime = MinTime + (MaxTime - InTime);
		}
	}

	return InTime;
}

template<typename KeyTimeAdapterType, typename KeyDataAdapterType>
static float InterpEvalExtrapolate(float InTime, const KeyTimeAdapterType& KeyTimeAdapter, const KeyDataAdapterType& KeyDataAdapter, ERichCurveExtrapolation InfinityExtrap, int32 KeyIndex0, int32 KeyIndex1, float KeyTime0)
{
	// For Pre-infinity, key0 and key1 are the actual key 0 and key 1
	// For Post-infinity, key0 and key1 are the last and second to last key
	const KeyDataHandle ValueHandle0 = KeyDataAdapter.GetKeyDataHandle(KeyIndex0);
	const float KeyValue0 = KeyDataAdapter.GetKeyValue(ValueHandle0);

	if (InfinityExtrap == RCCE_Linear)
	{
		const float KeyTime1 = KeyTimeAdapter.GetTime(KeyIndex1);
		const float DT = KeyTime1 - KeyTime0;

		if (FMath::IsNearlyZero(DT))
		{
			return KeyValue0;
		}
		else
		{
			const KeyDataHandle ValueHandle1 = KeyDataAdapter.GetKeyDataHandle(KeyIndex1);
			const float KeyValue1 = KeyDataAdapter.GetKeyValue(ValueHandle1);
			const float DV = KeyValue1 - KeyValue0;
			const float Slope = DV / DT;

			return Slope * (InTime - KeyTime0) + KeyValue0;
		}
	}
	else
	{
		// Otherwise if constant or in a cycle or oscillate, always use the first key value
		return KeyValue0;
	}
}

// Each template permutation is only called from one place, force inline it to avoid issues
template<typename KeyTimeAdapterType, typename KeyDataAdapterType>
FORCEINLINE_DEBUGGABLE static float InterpEval(float InTime, const KeyTimeAdapterType& KeyTimeAdapter, const KeyDataAdapterType& KeyDataAdapter, int32 NumKeys, ERichCurveExtrapolation PreInfinityExtrap, ERichCurveExtrapolation PostInfinityExtrap)
{
	float CycleValueOffset = 0.0;

	const float FirstKeyTime = KeyTimeAdapter.GetTime(0);
	if (InTime < FirstKeyTime)
	{
		if (PreInfinityExtrap != RCCE_Linear && PreInfinityExtrap != RCCE_Constant)
		{
			InTime = RemapTimeValue(InTime, KeyTimeAdapter, KeyDataAdapter, NumKeys, PreInfinityExtrap, 0, NumKeys - 1, CycleValueOffset);
		}
		else
		{
			return InterpEvalExtrapolate(InTime, KeyTimeAdapter, KeyDataAdapter, PreInfinityExtrap, 0, 1, FirstKeyTime);
		}
	}

	const float LastKeyTime = KeyTimeAdapter.GetTime(NumKeys - 1);
	if (InTime > LastKeyTime)
	{
		if (PostInfinityExtrap != RCCE_Linear && PostInfinityExtrap != RCCE_Constant)
		{
			InTime = RemapTimeValue(InTime, KeyTimeAdapter, KeyDataAdapter, NumKeys, PostInfinityExtrap, NumKeys - 1, 0, CycleValueOffset);
		}
		else
		{
			return InterpEvalExtrapolate(InTime, KeyTimeAdapter, KeyDataAdapter, PostInfinityExtrap, NumKeys - 1, NumKeys - 2, LastKeyTime);
		}
	}

	// perform a lower bound to get the second of the interpolation nodes
	int32 First = 1;
	int32 Last = NumKeys - 1;
	int32 Count = Last - First;

	while (Count > 0)
	{
		const int32 Step = Count / 2;
		const int32 Middle = First + Step;

		// TODO: Can we do the search with integers? In order to do so, we need to Floorf(..) the key times
		const float KeyTime = KeyTimeAdapter.GetTime(Middle);
		if (InTime >= KeyTime)
		{
			First = Middle + 1;
			Count -= Step + 1;
		}
		else
		{
			Count = Step;
		}
	}

	const float KeyTime0 = KeyTimeAdapter.GetTime(First - 1);
	const float KeyTime1 = KeyTimeAdapter.GetTime(First);
	const float Diff = KeyTime1 - KeyTime0;

	const KeyDataHandle KeyValueHandle0 = KeyDataAdapter.GetKeyDataHandle(First - 1);
	const float KeyValue0 = KeyDataAdapter.GetKeyValue(KeyValueHandle0);

	const auto IsItNotWeighted = [&KeyDataAdapter](KeyDataHandle Handle0, KeyDataHandle Handle1) -> bool
	{
		const ERichCurveTangentWeightMode Mode0 = KeyDataAdapter.GetKeyTangentWeightMode(Handle0);
		const ERichCurveTangentWeightMode Mode1 = KeyDataAdapter.GetKeyTangentWeightMode(Handle1);
		
		return (Mode0 == RCTWM_WeightedNone || Mode0 == RCTWM_WeightedArrive) && (Mode1 == RCTWM_WeightedNone || Mode1 == RCTWM_WeightedLeave);
	};

	// const value here allows the code to be stripped statically if the data is uniform
	// which it is most of the time
	const ERichCurveCompressionFormat KeyInterpMode0 = KeyDataAdapter.GetKeyInterpMode(First - 1);
	float InterpolatedValue;
	if (Diff > 0.0f)
	{
		const KeyDataHandle KeyValueHandle1 = KeyDataAdapter.GetKeyDataHandle(First);
		const float KeyValue1 = KeyDataAdapter.GetKeyValue(KeyValueHandle1);

		const float Alpha = (InTime - KeyTime0) / Diff;

		if (KeyInterpMode0 != RCCF_Constant)
        {
		    const float P0 = KeyValue0;
		    const float P3 = KeyValue1;
    
		    if (KeyInterpMode0 == RCCF_Linear)
		    {
			    InterpolatedValue = FMath::Lerp(P0, P3, Alpha);
		    }
		    else if (IsItNotWeighted(KeyValueHandle0, KeyValueHandle1))
		    {
			    const float OneThird = 1.0f / 3.0f;
			    const float ScaledDiff = Diff * OneThird;
			    const float KeyLeaveTangent0 = KeyDataAdapter.GetKeyLeaveTangent(KeyValueHandle0);
			    const float KeyArriveTangent1 = KeyDataAdapter.GetKeyArriveTangent(KeyValueHandle1);
			    const float P1 = P0 + (KeyLeaveTangent0 * ScaledDiff);
			    const float P2 = P3 - (KeyArriveTangent1 * ScaledDiff);
    
			    InterpolatedValue = UE::Curves::BezierInterp(P0, P1, P2, P3, Alpha);
		    }
		    else // it's weighted
		    {
			    return UE::Curves::WeightedEvalForTwoKeys(
				    KeyValue0, KeyTime0, KeyDataAdapter.GetKeyLeaveTangent(KeyValueHandle0), KeyDataAdapter.GetKeyLeaveTangentWeight(KeyValueHandle0), KeyDataAdapter.GetKeyTangentWeightMode(First - 1),
				    KeyValue1, KeyTime1, KeyDataAdapter.GetKeyArriveTangent(KeyValueHandle1), KeyDataAdapter.GetKeyArriveTangentWeight(KeyValueHandle1), KeyDataAdapter.GetKeyTangentWeightMode(First),
				    InTime);
		    }
		}
        else
        {
			InterpolatedValue = InTime < KeyTime1 ? KeyValue0 : KeyValue1;
        }
	}
	else
	{
		InterpolatedValue = KeyValue0;
	}

	return InterpolatedValue + CycleValueOffset;
}

namespace UE::Anim::RichCurve::Private
{
	typedef float(*InterpEvalFun)(const FCompressedRichCurve::FStaticEvalArgs& Args);

	// RCCF_Empty
	static float StaticEval_Empty(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		return Args.ConstantValueNumKeys.ConstantValue == MAX_flt ? Args.DefaultValue : Args.ConstantValueNumKeys.ConstantValue;
	}

	// RCCF_Constant
	static float StaticEval_Constant(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		return Args.ConstantValueNumKeys.ConstantValue;
	}

	// RCCF_Linear & RCKTCF_uint16
	static float StaticEval_Linear_u16(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		const int32 KeyTimesOffset = 0;
		Quantized16BitKeyTimeAdapter KeyTimeAdapter(Args.CompressedKeys, KeyTimesOffset, Args.ConstantValueNumKeys.NumKeys);
		UniformKeyDataAdapter<RCCF_Linear> KeyDataAdapter(Args.CompressedKeys, KeyTimeAdapter);
		return InterpEval(Args.Time, KeyTimeAdapter, KeyDataAdapter, Args.ConstantValueNumKeys.NumKeys, Args.PreInfinityExtrap, Args.PostInfinityExtrap);
	}

	// RCCF_Linear & RCKTCF_float32
	static float StaticEval_Linear_f32(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		const int32 KeyTimesOffset = 0;
		Float32BitKeyTimeAdapter KeyTimeAdapter(Args.CompressedKeys, KeyTimesOffset, Args.ConstantValueNumKeys.NumKeys);
		UniformKeyDataAdapter<RCCF_Linear> KeyDataAdapter(Args.CompressedKeys, KeyTimeAdapter);
		return InterpEval(Args.Time, KeyTimeAdapter, KeyDataAdapter, Args.ConstantValueNumKeys.NumKeys, Args.PreInfinityExtrap, Args.PostInfinityExtrap);
	}

	// RCCF_Cubic & RCKTCF_uint16
	static float StaticEval_Cubic_u16(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		const int32 KeyTimesOffset = 0;
		Quantized16BitKeyTimeAdapter KeyTimeAdapter(Args.CompressedKeys, KeyTimesOffset, Args.ConstantValueNumKeys.NumKeys);
		UniformKeyDataAdapter<RCCF_Cubic> KeyDataAdapter(Args.CompressedKeys, KeyTimeAdapter);
		return InterpEval(Args.Time, KeyTimeAdapter, KeyDataAdapter, Args.ConstantValueNumKeys.NumKeys, Args.PreInfinityExtrap, Args.PostInfinityExtrap);
	}

	// RCCF_Cubic & RCKTCF_float32
	static float StaticEval_Cubic_f32(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		const int32 KeyTimesOffset = 0;
		Float32BitKeyTimeAdapter KeyTimeAdapter(Args.CompressedKeys, KeyTimesOffset, Args.ConstantValueNumKeys.NumKeys);
		UniformKeyDataAdapter<RCCF_Cubic> KeyDataAdapter(Args.CompressedKeys, KeyTimeAdapter);
		return InterpEval(Args.Time, KeyTimeAdapter, KeyDataAdapter, Args.ConstantValueNumKeys.NumKeys, Args.PreInfinityExtrap, Args.PostInfinityExtrap);
	}

	// RCCF_Mixed & RCKTCF_uint16
	static float StaticEval_Mixed_u16(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		const int32 InterpModesOffset = 0;
		const int32 KeyTimesOffset = InterpModesOffset + Align(Args.ConstantValueNumKeys.NumKeys * sizeof(uint8), sizeof(uint16));
		Quantized16BitKeyTimeAdapter KeyTimeAdapter(Args.CompressedKeys, KeyTimesOffset, Args.ConstantValueNumKeys.NumKeys);
		MixedKeyDataAdapter KeyDataAdapter(Args.CompressedKeys, InterpModesOffset, KeyTimeAdapter);
		return InterpEval(Args.Time, KeyTimeAdapter, KeyDataAdapter, Args.ConstantValueNumKeys.NumKeys, Args.PreInfinityExtrap, Args.PostInfinityExtrap);
	}

	// RCCF_Mixed & RCKTCF_float32
	static float StaticEval_Mixed_f32(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		const int32 InterpModesOffset = 0;
		const int32 KeyTimesOffset = InterpModesOffset + Align(Args.ConstantValueNumKeys.NumKeys * sizeof(uint8), sizeof(float));
		Float32BitKeyTimeAdapter KeyTimeAdapter(Args.CompressedKeys, KeyTimesOffset, Args.ConstantValueNumKeys.NumKeys);
		MixedKeyDataAdapter KeyDataAdapter(Args.CompressedKeys, InterpModesOffset, KeyTimeAdapter);
		return InterpEval(Args.Time, KeyTimeAdapter, KeyDataAdapter, Args.ConstantValueNumKeys.NumKeys, Args.PreInfinityExtrap, Args.PostInfinityExtrap);
	}

	// RCCF_Weighted & RCKTCF_uint16
	static float StaticEval_Weighted_u16(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		const int32 InterpModesOffset = 0;
		const int32 KeyTimesOffset = InterpModesOffset + Align(2 * Args.ConstantValueNumKeys.NumKeys * sizeof(uint8), sizeof(uint16));
		Quantized16BitKeyTimeAdapter KeyTimeAdapter(Args.CompressedKeys, KeyTimesOffset, Args.ConstantValueNumKeys.NumKeys);
		WeightedKeyDataAdapter KeyDataAdapter(Args.CompressedKeys, InterpModesOffset, KeyTimeAdapter);
		return InterpEval(Args.Time, KeyTimeAdapter, KeyDataAdapter, Args.ConstantValueNumKeys.NumKeys, Args.PreInfinityExtrap, Args.PostInfinityExtrap);
	}

	// RCCF_Weighted & RCKTCF_float32
	static float StaticEval_Weighted_f32(const FCompressedRichCurve::FStaticEvalArgs& Args)
	{
		const int32 InterpModesOffset = 0;
		const int32 KeyTimesOffset = InterpModesOffset + Align(2 * Args.ConstantValueNumKeys.NumKeys * sizeof(uint8), sizeof(float));
		Float32BitKeyTimeAdapter KeyTimeAdapter(Args.CompressedKeys, KeyTimesOffset, Args.ConstantValueNumKeys.NumKeys);
		WeightedKeyDataAdapter KeyDataAdapter(Args.CompressedKeys, InterpModesOffset, KeyTimeAdapter);
		return InterpEval(Args.Time, KeyTimeAdapter, KeyDataAdapter, Args.ConstantValueNumKeys.NumKeys, Args.PreInfinityExtrap, Args.PostInfinityExtrap);
	}

	// For performance reasons, we store a map of template optimized functions to evaluate rich curves
	// Our array is two dimensional: ERichCurveCompressionFormat x ERichCurveKeyTimeCompressionFormat
	static InterpEvalFun InterpEvalMap[6][2]
	{
		// RCCF_Empty
		{
			&StaticEval_Empty,			// RCKTCF_uint16
			&StaticEval_Empty,			// RCKTCF_float32
		},
		// RCCF_Constant
		{
			&StaticEval_Constant,		// RCKTCF_uint16
			&StaticEval_Constant,		// RCKTCF_float32
		},
		// RCCF_Linear
		{
			&StaticEval_Linear_u16,		// RCKTCF_uint16
			&StaticEval_Linear_f32,		// RCKTCF_float32
		},
		// RCCF_Cubic
		{
			&StaticEval_Cubic_u16,		// RCKTCF_uint16
			&StaticEval_Cubic_f32,		// RCKTCF_float32
		},
		// RCCF_Mixed
		{
			&StaticEval_Mixed_u16,		// RCKTCF_uint16
			&StaticEval_Mixed_f32,		// RCKTCF_float32
		},
		// RCCF_Weighted
		{
			&StaticEval_Weighted_u16,	// RCKTCF_uint16
			&StaticEval_Weighted_f32,	// RCKTCF_float32
		},
	};
}

ERichCurveInterpMode GetKeyInterpMode(ERichCurveCompressionFormat Format)
{
	switch (Format)
	{
	case ERichCurveCompressionFormat::RCCF_Linear:
		return ERichCurveInterpMode::RCIM_Linear;
		
	case ERichCurveCompressionFormat::RCCF_Cubic:
		return ERichCurveInterpMode::RCIM_Cubic;
		
	case ERichCurveCompressionFormat::RCCF_Weighted:
		return ERichCurveInterpMode::RCIM_Cubic;

	case ERichCurveCompressionFormat::RCCF_Constant:
		return ERichCurveInterpMode::RCIM_Constant;

	case ERichCurveCompressionFormat::RCCF_Empty:
		return ERichCurveInterpMode::RCIM_None;

	case ERichCurveCompressionFormat::RCCF_Mixed:
		check(false);		
	}
	
	return ERichCurveInterpMode::RCIM_None;
}

static TFunction<FRichCurveKey(FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)> ToRichCurveMap[6][2]
{
	// RCCF_Empty
	{
		// RCKTCF_uint16
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			return FRichCurveKey(0.f, ConstantValueNumKeys.ConstantValue);
		},
		// RCKTCF_float32
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			return FRichCurveKey(0.f, ConstantValueNumKeys.ConstantValue);
		},
	},
	// RCCF_Constant
	{
		// RCKTCF_uint16
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			return FRichCurveKey(0.f, ConstantValueNumKeys.ConstantValue);
		},
		// RCKTCF_float32
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			return FRichCurveKey(0.f, ConstantValueNumKeys.ConstantValue);
		},
	},
	// RCCF_Linear
	{
		// RCKTCF_uint16
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			const int32 KeyTimesOffset = 0;
			const Quantized16BitKeyTimeAdapter KeyTimeAdapter(CompressedKeys, KeyTimesOffset, ConstantValueNumKeys.NumKeys);
			const UniformKeyDataAdapter<RCCF_Linear> KeyDataAdapter(CompressedKeys, KeyTimeAdapter);

			const KeyDataHandle Handle = KeyDataAdapter.GetKeyDataHandle(Index);
			return FRichCurveKey(KeyTimeAdapter.GetTime(Index), KeyDataAdapter.GetKeyValue(Handle));
		},
		// RCKTCF_float32
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			const int32 KeyTimesOffset = 0;
			const Float32BitKeyTimeAdapter KeyTimeAdapter(CompressedKeys, KeyTimesOffset, ConstantValueNumKeys.NumKeys);
			const UniformKeyDataAdapter<RCCF_Linear> KeyDataAdapter(CompressedKeys, KeyTimeAdapter);

			const KeyDataHandle Handle = KeyDataAdapter.GetKeyDataHandle(Index);
			return FRichCurveKey(KeyTimeAdapter.GetTime(Index), KeyDataAdapter.GetKeyValue(Handle));
		},
	},
	// RCCF_Cubic
	{
		// RCKTCF_uint16
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			const int32 KeyTimesOffset = 0;
			const Quantized16BitKeyTimeAdapter KeyTimeAdapter(CompressedKeys, KeyTimesOffset, ConstantValueNumKeys.NumKeys);
			const UniformKeyDataAdapter<RCCF_Cubic> KeyDataAdapter(CompressedKeys, KeyTimeAdapter);

			const KeyDataHandle Handle = KeyDataAdapter.GetKeyDataHandle(Index);
			return FRichCurveKey(KeyTimeAdapter.GetTime(Index), KeyDataAdapter.GetKeyValue(Handle), KeyDataAdapter.GetKeyArriveTangent(Handle), KeyDataAdapter.GetKeyLeaveTangent(Handle), RCIM_Cubic);
		},
		// RCKTCF_float32
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			const int32 KeyTimesOffset = 0;
			const Float32BitKeyTimeAdapter KeyTimeAdapter(CompressedKeys, KeyTimesOffset, ConstantValueNumKeys.NumKeys);
			const UniformKeyDataAdapter<RCCF_Cubic> KeyDataAdapter(CompressedKeys, KeyTimeAdapter);

			const KeyDataHandle Handle = KeyDataAdapter.GetKeyDataHandle(Index);
			return FRichCurveKey(KeyTimeAdapter.GetTime(Index), KeyDataAdapter.GetKeyValue(Handle), KeyDataAdapter.GetKeyArriveTangent(Handle), KeyDataAdapter.GetKeyLeaveTangent(Handle), RCIM_Cubic);
		},
	},
	// RCCF_Mixed
	{
		// RCKTCF_uint16
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			const int32 InterpModesOffset = 0;
			const int32 KeyTimesOffset = InterpModesOffset + Align(ConstantValueNumKeys.NumKeys * sizeof(uint8), sizeof(uint16));
			const Quantized16BitKeyTimeAdapter KeyTimeAdapter(CompressedKeys, KeyTimesOffset, ConstantValueNumKeys.NumKeys);
			const MixedKeyDataAdapter KeyDataAdapter(CompressedKeys, InterpModesOffset, KeyTimeAdapter);

			const KeyDataHandle Handle = KeyDataAdapter.GetKeyDataHandle(Index);			
			const FRichCurveKey Key(KeyTimeAdapter.GetTime(Index), KeyDataAdapter.GetKeyValue(Handle), KeyDataAdapter.GetKeyArriveTangent(Handle), KeyDataAdapter.GetKeyLeaveTangent(Handle),
				GetKeyInterpMode(KeyDataAdapter.GetKeyInterpMode(Index)));
			
			return Key;
		},
		// RCKTCF_float32
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			const int32 InterpModesOffset = 0;
			const int32 KeyTimesOffset = InterpModesOffset + Align(ConstantValueNumKeys.NumKeys * sizeof(uint8), sizeof(float));
			const Float32BitKeyTimeAdapter KeyTimeAdapter(CompressedKeys, KeyTimesOffset, ConstantValueNumKeys.NumKeys);
			const MixedKeyDataAdapter KeyDataAdapter(CompressedKeys, InterpModesOffset, KeyTimeAdapter);

			const KeyDataHandle Handle = KeyDataAdapter.GetKeyDataHandle(Index);			
			FRichCurveKey Key(KeyTimeAdapter.GetTime(Index), KeyDataAdapter.GetKeyValue(Handle), KeyDataAdapter.GetKeyArriveTangent(Handle), KeyDataAdapter.GetKeyLeaveTangent(Handle),
				GetKeyInterpMode(KeyDataAdapter.GetKeyInterpMode(Index)));
			
			return Key;
		},
	},
	// RCCF_Weighted
	{
		// RCKTCF_uint16
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			const int32 InterpModesOffset = 0;
			const int32 KeyTimesOffset = InterpModesOffset + Align(2 * ConstantValueNumKeys.NumKeys * sizeof(uint8), sizeof(uint16));
			const Quantized16BitKeyTimeAdapter KeyTimeAdapter(CompressedKeys, KeyTimesOffset, ConstantValueNumKeys.NumKeys);
			const WeightedKeyDataAdapter KeyDataAdapter(CompressedKeys, InterpModesOffset, KeyTimeAdapter);

			const KeyDataHandle Handle = KeyDataAdapter.GetKeyDataHandle(Index);
			FRichCurveKey Key(KeyTimeAdapter.GetTime(Index), KeyDataAdapter.GetKeyValue(Handle), KeyDataAdapter.GetKeyArriveTangent(Handle), KeyDataAdapter.GetKeyLeaveTangent(Handle),
				GetKeyInterpMode(KeyDataAdapter.GetKeyInterpMode(Index)));

			Key.TangentMode = ERichCurveTangentMode::RCTM_User;
			Key.TangentWeightMode = KeyDataAdapter.GetKeyTangentWeightMode(Index);
			Key.ArriveTangentWeight = KeyDataAdapter.GetKeyArriveTangentWeight(Handle);
			Key.LeaveTangentWeight = KeyDataAdapter.GetKeyLeaveTangentWeight(Handle);
			
			return Key;
		},
		// RCKTCF_float32
		[](FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, int32 Index)
		{
			const int32 InterpModesOffset = 0;
			const int32 KeyTimesOffset = InterpModesOffset + Align(2 * ConstantValueNumKeys.NumKeys * sizeof(uint8), sizeof(float));
			const Float32BitKeyTimeAdapter KeyTimeAdapter(CompressedKeys, KeyTimesOffset, ConstantValueNumKeys.NumKeys);
			const WeightedKeyDataAdapter KeyDataAdapter(CompressedKeys, InterpModesOffset, KeyTimeAdapter);

			const KeyDataHandle Handle = KeyDataAdapter.GetKeyDataHandle(Index);
			
			FRichCurveKey Key(KeyTimeAdapter.GetTime(Index), KeyDataAdapter.GetKeyValue(Handle), KeyDataAdapter.GetKeyArriveTangent(Handle), KeyDataAdapter.GetKeyLeaveTangent(Handle),
				GetKeyInterpMode(KeyDataAdapter.GetKeyInterpMode(Index)));

			Key.TangentMode = ERichCurveTangentMode::RCTM_User;
			Key.TangentWeightMode = KeyDataAdapter.GetKeyTangentWeightMode(Index);
			Key.ArriveTangentWeight = KeyDataAdapter.GetKeyArriveTangentWeight(Handle);
			Key.LeaveTangentWeight = KeyDataAdapter.GetKeyLeaveTangentWeight(Handle);

			return Key;
		},
	},
};

void FCompressedRichCurve::PopulateCurve(FRichCurve& OutCurve) const
{
	if (CompressionFormat != RCCF_Constant)
	{
		TArray<FRichCurveKey> Keys;
		for (int32 KeyIndex = 0; KeyIndex < ConstantValueNumKeys.NumKeys; ++KeyIndex)
		{
			Keys.Add(ToRichCurveMap[CompressionFormat][KeyTimeCompressionFormat](ConstantValueNumKeys, CompressedKeys.GetData(), KeyIndex));
		}
		OutCurve.SetKeys(Keys);
	}
}

float FCompressedRichCurve::Eval(float InTime, float InDefaultValue) const
{
	FStaticEvalArgs Args;
	Args.CompressionFormat = CompressionFormat;
	Args.KeyTimeCompressionFormat = KeyTimeCompressionFormat;
	Args.PreInfinityExtrap = PreInfinityExtrap;
	Args.PostInfinityExtrap = PostInfinityExtrap;
	Args.ConstantValueNumKeys = ConstantValueNumKeys;
	Args.CompressedKeys = CompressedKeys.GetData();
	Args.Time = InTime;
	Args.DefaultValue = InDefaultValue;

	return StaticEval(Args);
}

float FCompressedRichCurve::StaticEval(ERichCurveCompressionFormat CompressionFormat, ERichCurveKeyTimeCompressionFormat KeyTimeCompressionFormat, ERichCurveExtrapolation PreInfinityExtrap, ERichCurveExtrapolation PostInfinityExtrap, FCompressedRichCurve::TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, float InTime, float InDefaultValue)
{
	FStaticEvalArgs Args;
	Args.CompressionFormat = CompressionFormat;
	Args.KeyTimeCompressionFormat = KeyTimeCompressionFormat;
	Args.PreInfinityExtrap = PreInfinityExtrap;
	Args.PostInfinityExtrap = PostInfinityExtrap;
	Args.ConstantValueNumKeys = ConstantValueNumKeys;
	Args.CompressedKeys = CompressedKeys;
	Args.Time = InTime;
	Args.DefaultValue = InDefaultValue;

	return StaticEval(Args);
}

float FCompressedRichCurve::StaticEval(const FStaticEvalArgs& Args)
{
	// Dynamic dispatch into a template optimized code path
	const UE::Anim::RichCurve::Private::InterpEvalFun EvalFun = UE::Anim::RichCurve::Private::InterpEvalMap[Args.CompressionFormat][Args.KeyTimeCompressionFormat];
	const float Value = (*EvalFun)(Args);
	return Value;
}

bool FCompressedRichCurve::Serialize(FArchive& Ar)
{
	Ar << CompressionFormat;
	Ar << KeyTimeCompressionFormat;
	Ar << PreInfinityExtrap;
	Ar << PostInfinityExtrap;

	int32 NumKeysOrConstant = ConstantValueNumKeys.NumKeys;
	Ar << NumKeysOrConstant;
	ConstantValueNumKeys.NumKeys = NumKeysOrConstant;

	if (Ar.IsLoading())
	{
		int32 NumBytes;
		Ar << NumBytes;

		CompressedKeys.Empty(NumBytes);
		CompressedKeys.AddUninitialized(NumBytes);
		Ar.Serialize(CompressedKeys.GetData(), NumBytes);
	}
	else
	{
		int32 NumBytes = CompressedKeys.Num();
		Ar << NumBytes;
		Ar.Serialize(CompressedKeys.GetData(), NumBytes);
	}

	return true;
}

bool FCompressedRichCurve::operator==(const FCompressedRichCurve& Other) const
{
	return CompressionFormat == Other.CompressionFormat
		&& KeyTimeCompressionFormat == Other.KeyTimeCompressionFormat
		&& PreInfinityExtrap == Other.PreInfinityExtrap
		&& PostInfinityExtrap == Other.PostInfinityExtrap
		&& ConstantValueNumKeys.NumKeys == Other.ConstantValueNumKeys.NumKeys
		&& CompressedKeys == Other.CompressedKeys;
}
