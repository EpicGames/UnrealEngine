// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/SimpleCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleCurve)

/* FSimpleCurveKey interface
 *****************************************************************************/

bool FSimpleCurveKey::Serialize(FArchive& Ar)
{
	// Serialization is handled manually to avoid the extra size overhead of FProperty tagging.
	// Otherwise with many keys in a rich curve the size can become quite large.
	Ar << Time;
	Ar << Value;

	return true;
}


bool FSimpleCurveKey::operator==( const FSimpleCurveKey& Curve ) const
{
	return (Time == Curve.Time) && (Value == Curve.Value);
}

bool FSimpleCurveKey::operator!=(const FSimpleCurveKey& Other) const
{
	return !(*this == Other);
}

/* FSimpleCurve interface
 *****************************************************************************/

TArray<FSimpleCurveKey> FSimpleCurve::GetCopyOfKeys() const
{
	return Keys;
}

const TArray<FSimpleCurveKey>& FSimpleCurve::GetConstRefOfKeys() const
{
	return Keys;
}

TArray<FSimpleCurveKey>::TConstIterator FSimpleCurve::GetKeyIterator() const
{
	return Keys.CreateConstIterator();
}

FSimpleCurveKey& FSimpleCurve::GetKey(FKeyHandle KeyHandle)
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}

FSimpleCurveKey FSimpleCurve::GetKey(FKeyHandle KeyHandle) const
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}

FSimpleCurveKey FSimpleCurve::GetFirstKey() const
{
	check(Keys.Num() > 0);
	return Keys[0];
}

FSimpleCurveKey FSimpleCurve::GetLastKey() const
{
	check(Keys.Num() > 0);
	return Keys[Keys.Num()-1];
}

FSimpleCurveKey* FSimpleCurve::GetFirstMatchingKey(const TArray<FKeyHandle>& KeyHandles)
{
	for (const FKeyHandle& KeyHandle : KeyHandles)
	{
		if (IsKeyHandleValid(KeyHandle))
		{
			return &GetKey(KeyHandle);
		}
	}

	return nullptr;
}

FKeyHandle FSimpleCurve::AddKey( const float InTime, const float InValue, const bool bUnwindRotation, FKeyHandle NewHandle )
{
	int32 Index = 0;
	for(; Index < Keys.Num() && Keys[Index].Time < InTime; ++Index);
	Keys.Insert(FSimpleCurveKey(InTime, InValue), Index);

	// If we were asked to treat this curve as a rotation value and to unwindow the rotation, then
	// we'll look at the previous key and modify the key's value to use a rotation angle that is
	// continuous with the previous key while retaining the exact same rotation angle, if at all necessary
	if( Index > 0 && bUnwindRotation )
	{
		const float OldValue = Keys[ Index - 1 ].Value;
		float NewValue = Keys[ Index ].Value;

		while( NewValue - OldValue > 180.0f )
		{
			NewValue -= 360.0f;
		}
		while( NewValue - OldValue < -180.0f )
		{
			NewValue += 360.0f;
		}

		Keys[Index].Value = NewValue;
	}
	
	KeyHandlesToIndices.Add(NewHandle, Index);

	return NewHandle;
}


void FSimpleCurve::SetKeys(const TArray<FSimpleCurveKey>& InKeys)
{
	Reset();

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Keys.Add(InKeys[Index]);
		KeyHandlesToIndices.Add(FKeyHandle(), Index);
	}
}


void FSimpleCurve::DeleteKey(FKeyHandle InKeyHandle)
{
	int32 Index = GetIndex(InKeyHandle);
	
	Keys.RemoveAt(Index);

	KeyHandlesToIndices.Remove(InKeyHandle);
}


FKeyHandle FSimpleCurve::UpdateOrAddKey(float InTime, float InValue, const bool bUnwindRotation, float KeyTimeTolerance)
{
	// Search for a key that already exists at the time and if found, update its value
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		float KeyTime = Keys[KeyIndex].Time;

		if (FMath::IsNearlyEqual(KeyTime, InTime, KeyTimeTolerance))
		{
			Keys[KeyIndex].Value = InValue;

			return GetKeyHandle(KeyIndex);
		}

		if (KeyTime > InTime)
		{
			// All the rest of the keys exist after the key we want to add
			// so there is no point in searching
			break;
		}
	}

	// A key wasn't found, add it now
	return AddKey(InTime, InValue, bUnwindRotation);
}


void FSimpleCurve::SetKeyTime( FKeyHandle KeyHandle, float NewTime )
{
	if (IsKeyHandleValid(KeyHandle))
	{
		const FSimpleCurveKey OldKey = GetKey(KeyHandle);

		DeleteKey(KeyHandle);
		AddKey(NewTime, OldKey.Value, false, KeyHandle);
	}
}


float FSimpleCurve::GetKeyTime(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Time;
}


int32 FSimpleCurve::GetKeyIndex(float KeyTime, float KeyTimeTolerance) const
{
	int32 Start = 0;
	int32 End = Keys.Num()-1;

	// Binary search since the keys are in sorted order
	while (Start <= End)
	{
		int32 TestPos = Start + (End-Start) / 2;
		float TestKeyTime = Keys[TestPos].Time;

		if (FMath::IsNearlyEqual(TestKeyTime, KeyTime, KeyTimeTolerance))
		{
			return TestPos;
		}
		else if(TestKeyTime < KeyTime)
		{
			Start = TestPos+1;
		}
		else
		{
			End = TestPos-1;
		}
	}

	return INDEX_NONE;
}


void FSimpleCurve::SetKeyValue(FKeyHandle KeyHandle, float NewValue, bool /*bAutoSetTangents*/)
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return;
	}

	GetKey(KeyHandle).Value = NewValue;
}


float FSimpleCurve::GetKeyValue(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Value;
}

TPair<float,float> FSimpleCurve::GetKeyTimeValuePair(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return TPair<float, float>(0.f, 0.f);
	}

	const FSimpleCurveKey& Key = GetKey(KeyHandle);
	return TPair<float,float>(Key.Time, Key.Value);
}

void FSimpleCurve::GetTimeRange(float& MinTime, float& MaxTime) const
{
	if (Keys.Num() == 0)
	{
		MinTime = 0.f;
		MaxTime = 0.f;
	}
	else
	{
		MinTime = Keys[0].Time;
		MaxTime = Keys[Keys.Num()-1].Time;
	}
}

void FSimpleCurve::GetValueRange(float& MinValue, float& MaxValue) const
{
	if (Keys.Num() == 0)
	{
		MinValue = MaxValue = 0.f;
	}
	else
	{
		MinValue = MaxValue = Keys[0].Value;

		for (int32 i = 0; i < Keys.Num(); i++)
		{
			const FSimpleCurveKey& Key = Keys[i];

			MinValue = FMath::Min(MinValue, Key.Value);
			MaxValue = FMath::Max(MaxValue, Key.Value);
		}
	}
}

void FSimpleCurve::Reset()
{
	Keys.Empty();
	KeyHandlesToIndices.Empty();
}

void FSimpleCurve::ReadjustTimeRange(float NewMinTimeRange, float NewMaxTimeRange, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime)
{
	// first readjust modified time keys
	float ModifiedDuration = OldEndTime - OldStartTime;

	if (bInsert)
	{
		for(int32 KeyIndex=0; KeyIndex<Keys.Num(); ++KeyIndex)
		{
			float& CurrentTime = Keys[KeyIndex].Time;
			if (CurrentTime >= OldStartTime)
			{
				CurrentTime += ModifiedDuration;
			}
		}
	}
	else
	{
		// since we only allow one key at a given time, we will just cache the value that needs to be saved
		// this is the key to be replaced when this section is gone
		bool bAddNewKey = false; 
		float NewValue = 0.f;
		TArray<int32> KeysToDelete;

		for(int32 KeyIndex=0; KeyIndex<Keys.Num(); ++KeyIndex)
		{
			float& CurrentTime = Keys[KeyIndex].Time;
			// if this key exists between range of deleted
			// we'll evaluate the value at the "OldStartTime"
			// and re-add key, so that it keeps the previous value at the
			// start time
			// But that means if there are multiple keys, 
			// since we don't want multiple values in the same time
			// the last one will override the value
			if( CurrentTime >= OldStartTime && CurrentTime <= OldEndTime)
			{
				// get new value and add new key on one of OldStartTime, OldEndTime;
				// this is a bit complicated problem since we don't know if OldStartTime or OldEndTime is preferred. 
				// generall we use OldEndTime unless OldStartTime == 0.f
				// which means it's cut in the beginning. Otherwise it will always use the end time. 
				bAddNewKey = true;
				if (OldStartTime != 0.f)
				{
					NewValue = Eval(OldStartTime);
				}
				else
				{
					NewValue = Eval(OldEndTime);
				}
				// remove this key, but later because it might change eval result
				KeysToDelete.Add(KeyIndex);
			}
			else if (CurrentTime > OldEndTime)
			{
				CurrentTime -= ModifiedDuration;
			}
		}

		if (bAddNewKey)
		{
			for (int32 KeyIndex = KeysToDelete.Num()-1; KeyIndex >= 0; --KeyIndex)
			{
				const FKeyHandle* KeyHandle = KeyHandlesToIndices.FindKey(KeysToDelete[KeyIndex]);
				if(KeyHandle)
				{
					DeleteKey(*KeyHandle);
				}
			}

			UpdateOrAddKey(OldStartTime, NewValue);
		}
	}

	// now remove all redundant key
	TArray<FSimpleCurveKey> NewKeys;
	Exchange(NewKeys, Keys);

	for(int32 KeyIndex=0; KeyIndex<NewKeys.Num(); ++KeyIndex)
	{
		UpdateOrAddKey(NewKeys[KeyIndex].Time, NewKeys[KeyIndex].Value);
	}

	// now cull out all out of range 
	float MinTime, MaxTime;
	GetTimeRange(MinTime, MaxTime);

	bool bNeedToDeleteKey=false;

	// if there is key below min time, just add key at new min range, 
	if (MinTime < NewMinTimeRange)
	{
		float NewValue = Eval(NewMinTimeRange);
		UpdateOrAddKey(NewMinTimeRange, NewValue);

		bNeedToDeleteKey = true;
	}

	// if there is key after max time, just add key at new max range, 
	if(MaxTime > NewMaxTimeRange)
	{
		float NewValue = Eval(NewMaxTimeRange);
		UpdateOrAddKey(NewMaxTimeRange, NewValue);

		bNeedToDeleteKey = true;
	}

	// delete the keys outside of range
	if (bNeedToDeleteKey)
	{
		for (int32 KeyIndex=0; KeyIndex<Keys.Num(); ++KeyIndex)
		{
			if (Keys[KeyIndex].Time < NewMinTimeRange || Keys[KeyIndex].Time > NewMaxTimeRange)
			{
				const FKeyHandle* KeyHandle = KeyHandlesToIndices.FindKey(KeyIndex);
				if (KeyHandle)
				{
					DeleteKey(*KeyHandle);
					--KeyIndex;
				}
			}
		}
	}
}

void FSimpleCurve::BakeCurve(float SampleRate)
{
	if (Keys.Num() == 0)
	{
		return;
	}

	float FirstKeyTime = Keys[0].Time;
	float LastKeyTime = Keys[Keys.Num()-1].Time;

	BakeCurve(SampleRate, FirstKeyTime, LastKeyTime);
}

void FSimpleCurve::BakeCurve(float SampleRate, float FirstKeyTime, float LastKeyTime)
{
	if (Keys.Num() == 0)
	{
		return;
	}

	// we need to generate new keys first rather than modifying the
	// curve directly since that would affect the results of Eval calls
	TArray<TPair<float, float> > BakedKeys;
	BakedKeys.Reserve(((LastKeyTime - FirstKeyTime) / SampleRate) - 1);

	// the skip the first and last key unchanged
	for (float Time = FirstKeyTime + SampleRate; Time < LastKeyTime; )
	{
		const float Value = Eval(Time);
		BakedKeys.Add(TPair<float, float>(Time, Value));
		Time += SampleRate;
	}

	for (const TPair<float, float>& NewKey : BakedKeys)
	{
		UpdateOrAddKey(NewKey.Key, NewKey.Value);
	}
}

void FSimpleCurve::RemoveRedundantKeys(float Tolerance, FFrameRate SampleRate /*= FFrameRate(0,0)*/ )
{
	if (Keys.Num() < 3)
	{
		return;
	}

	RemoveRedundantKeysInternal(Tolerance, 0, Keys.Num() - 1);
}

void FSimpleCurve::RemoveRedundantKeys(float Tolerance, float FirstKeyTime, float LastKeyTime, FFrameRate SampleRate /*= FFrameRate(0,0)*/ )
{
	if (FirstKeyTime >= LastKeyTime)
	{
		return;
	}
	int32 StartKey = -1;
	int32 EndKey = -1;
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		const float CurrentKeyTime = Keys[KeyIndex].Time;

		if (CurrentKeyTime <= FirstKeyTime)
		{
			StartKey = KeyIndex;
		}
		if (CurrentKeyTime >= LastKeyTime)
		{
			EndKey = KeyIndex;
			break;
		}
	}

	if ((StartKey != INDEX_NONE) && (EndKey != INDEX_NONE))
	{
		RemoveRedundantKeysInternal(Tolerance, StartKey, EndKey);
	}
}

float FSimpleCurve::EvalForTwoKeys(const FSimpleCurveKey& Key1, const FSimpleCurveKey& Key2, const float InTime) const
{
	const float Diff = Key2.Time - Key1.Time;

	if (Diff > 0.f && InterpMode != RCIM_Constant)
	{
		const float Alpha = (InTime - Key1.Time) / Diff;
		const float P0 = Key1.Value;
		const float P3 = Key2.Value;

		return FMath::Lerp(P0, P3, Alpha);
	}
	else
	{
		return Key1.Value;
	}
}

void FSimpleCurve::RemoveRedundantKeysInternal(float Tolerance, int32 InStartKeepKey, int32 InEndKeepKey)
{
	int32 NumKeys = Keys.Num();
	if (NumKeys < 3) // Will always keep first and last key
	{
		return;
	}

	const int32 ActualStartKeepKey = FMath::Max(InStartKeepKey, 0); // Will always keep first and last key
	const int32 ActualEndKeepKey = FMath::Min(InEndKeepKey, NumKeys - 1);

	check(ActualStartKeepKey < ActualEndKeepKey); // Make sure we are doing something sane
	if ((ActualEndKeepKey - ActualStartKeepKey) < 2)
	{
		//Not going to do anything useful
		return;
	}

	// Cook Determinism note:
	// RemoveRedundantKeysInternal can be called a variable number of times during a cook, due to
	// CompositeTables which might call it again after their Parent table has already called it.
	// To prevent that variable number of calls from non-deterministically removing a variable number
	// of keys, we need to make sure that calling RemoveRedundantKeysInternal a second time after it
	// was called once is a noop that does not chose to remove any more keys.

	// Keep all keys up to and including ActualStartKeepKey even if they are redundant
	TArray<int32> KeepIndices;
	KeepIndices.Reserve(NumKeys);
	int32 NextEvalIndex = 0;
	for (; NextEvalIndex <= ActualStartKeepKey; ++NextEvalIndex)
	{
		KeepIndices.Add(NextEvalIndex);
	}

	// Consider keys up to ActualEndKeepKey and remove them if they are redundant.
	auto ShouldKeepMiddleKey = [this, Tolerance](const FSimpleCurveKey* Start, const FSimpleCurveKey* Middle, const FSimpleCurveKey* End)
	{
		const float KeyValue = Middle->Value;
		const float ValueWithoutKey = EvalForTwoKeys(*Start, *End, Middle->Time);
		return FMath::Abs(ValueWithoutKey - KeyValue) > Tolerance;
	};

	check(KeepIndices.Num() >= 1); // ActualStartKeepKey is in the list
	for (;;)
	{
		// Find the next index we will keep
		bool bSkippedKeys = false;
		const FSimpleCurveKey* EvalStartKey = &Keys[KeepIndices.Last()];
		while (NextEvalIndex < ActualEndKeepKey)
		{
			const FSimpleCurveKey* EvalMiddleKey = &Keys[NextEvalIndex]; // Exists and is not ActualEndKeepKey
			const FSimpleCurveKey* EvalEndKey = &Keys[NextEvalIndex + 1]; // Exists because ActualEndKeepKey is later in the list
			if (ShouldKeepMiddleKey(EvalStartKey, EvalMiddleKey, EvalEndKey))
			{
				break;
			}
			++NextEvalIndex;
			bSkippedKeys = true;
		}

		if (bSkippedKeys)
		{
			bool bRemovedKeys = false;
			// If we skipped any keys, reevaluate the last keys added popping them until we find one to keep.
			// If we remove any then back up one and reevalue the NextEvalIndex we decided to keep.
			while (KeepIndices.Last() > ActualStartKeepKey) // ActualStartKeepKey and earlier are not removable
			{
				int32 NumKeepIndices = KeepIndices.Num();
				check(NumKeepIndices >= 2); // ActualStartKeepKey and the one we are going to evaluate are both in the list
				EvalStartKey = &Keys[KeepIndices[NumKeepIndices - 2]];
				const FSimpleCurveKey* EvalMiddleKey = &Keys[KeepIndices[NumKeepIndices - 1]];
				const FSimpleCurveKey* EvalEndKey = &Keys[NextEvalIndex];
				if (ShouldKeepMiddleKey(EvalStartKey, EvalMiddleKey, EvalEndKey))
				{
					break;
				}
				KeepIndices.Pop(EAllowShrinking::No);
				check(KeepIndices.Num() >= 1); // ActualStartKeepKey is in the list
				bRemovedKeys = true;
			}
			if (bRemovedKeys)
			{
				continue; // Reevaluate NextEvalIndex
			}
		}
		if (NextEvalIndex >= ActualEndKeepKey)
		{
			// No further removable indices, we are done evaluating
			break;
		}

		// Mark NextEvalIndex for keeping and move to the next key
		KeepIndices.Add(NextEvalIndex);
		++NextEvalIndex;
	}

	// Add end keys that we are keeping
	for (; NextEvalIndex < NumKeys; ++NextEvalIndex)
	{
		KeepIndices.Add(NextEvalIndex);
	}

	// Build some helper data for managing the KeyHandlesToIndices map
	TArray<FKeyHandle> AllHandlesByIndex;
	if (KeyHandlesToIndices.Num() != 0)
	{
		// Do not use AddDefaulted as every FKeyHandle constructor allocates a new global identifier
		AllHandlesByIndex.AddZeroed(NumKeys);

		check(KeyHandlesToIndices.Num() == NumKeys);
		for (const TPair<FKeyHandle, int32>& HandleIndexPair : KeyHandlesToIndices.GetMap())
		{
			AllHandlesByIndex[HandleIndexPair.Value] = HandleIndexPair.Key;
		}
	}
	else
	{
		AllHandlesByIndex.AddDefaulted(NumKeys);
	}

	// Copy keys and KeyHandlesToIndices from all the indices we decided to keep
	int32 NumKeepKeys = KeepIndices.Num();
	TArray<FSimpleCurveKey> NewKeys;
	NewKeys.Reserve(NumKeepKeys);
	KeyHandlesToIndices.Empty();
	for (int32 NewIndex = 0; NewIndex < NumKeepKeys; ++NewIndex)
	{
		int32 OldIndex = KeepIndices[NewIndex];
		NewKeys.Add(MoveTemp(Keys[OldIndex]));
		KeyHandlesToIndices.Add(AllHandlesByIndex[OldIndex], NewIndex);
	}
	Keys = MoveTemp(NewKeys);
}

void FSimpleCurve::RemapTimeValue(float& InTime, float& CycleValueOffset) const
{
	const int32 NumKeys = Keys.Num();

	if (NumKeys < 2)
	{
		return;
	}

	if (InTime <= Keys[0].Time)
	{
		if (PreInfinityExtrap != RCCE_Linear && PreInfinityExtrap != RCCE_Constant)
		{
			float MinTime = Keys[0].Time;
			float MaxTime = Keys[NumKeys - 1].Time;

			int CycleCount = 0;
			CycleTime(MinTime, MaxTime, InTime, CycleCount);

			if (PreInfinityExtrap == RCCE_CycleWithOffset)
			{
				float DV = Keys[0].Value - Keys[NumKeys - 1].Value;
				CycleValueOffset = DV * CycleCount;
			}
			else if (PreInfinityExtrap == RCCE_Oscillate)
			{
				if (CycleCount % 2 == 1)
				{
					InTime = MinTime + (MaxTime - InTime);
				}
			}
		}
	}
	else if (InTime >= Keys[NumKeys - 1].Time)
	{
		if (PostInfinityExtrap != RCCE_Linear && PostInfinityExtrap != RCCE_Constant)
		{
			float MinTime = Keys[0].Time;
			float MaxTime = Keys[NumKeys - 1].Time;

			int CycleCount = 0;
			CycleTime(MinTime, MaxTime, InTime, CycleCount);

			if (PostInfinityExtrap == RCCE_CycleWithOffset)
			{
				float DV = Keys[NumKeys - 1].Value - Keys[0].Value;
				CycleValueOffset = DV * CycleCount;
			}
			else if (PostInfinityExtrap == RCCE_Oscillate)
			{
				if (CycleCount % 2 == 1)
				{
					InTime = MinTime + (MaxTime - InTime);
				}
			}
		}
	}
}

float FSimpleCurve::Eval(float InTime, float InDefaultValue) const
{
	// Remap time if extrapolation is present and compute offset value to use if cycling 
	float CycleValueOffset = 0;
	RemapTimeValue(InTime, CycleValueOffset);

	const int32 NumKeys = Keys.Num();

	// If the default value hasn't been initialized, use the incoming default value
	float InterpVal = DefaultValue == MAX_flt ? InDefaultValue : DefaultValue;

	if (NumKeys == 0)
	{
		// If no keys in curve, return the Default value.
	}
	else if (NumKeys < 2 || (InTime <= Keys[0].Time))
	{
		if (PreInfinityExtrap == RCCE_Linear && NumKeys > 1)
		{
			float DT = Keys[1].Time - Keys[0].Time;

			if (FMath::IsNearlyZero(DT))
			{
				InterpVal = Keys[0].Value;
			}
			else
			{
				float DV = Keys[1].Value - Keys[0].Value;
				float Slope = DV / DT;

				InterpVal = Slope * (InTime - Keys[0].Time) + Keys[0].Value;
			}
		}
		else
		{
			// Otherwise if constant or in a cycle or oscillate, always use the first key value
			InterpVal = Keys[0].Value;
		}
	}
	else if (InTime < Keys[NumKeys - 1].Time)
	{
		// perform a lower bound to get the second of the interpolation nodes
		int32 first = 1;
		int32 last = NumKeys - 1;
		int32 count = last - first;

		while (count > 0)
		{
			int32 step = count / 2;
			int32 middle = first + step;

			if (InTime >= Keys[middle].Time)
			{
				first = middle + 1;
				count -= step + 1;
			}
			else
			{
				count = step;
			}
		}

		InterpVal = EvalForTwoKeys(Keys[first - 1], Keys[first], InTime);
	}
	else
	{
		if (PostInfinityExtrap == RCCE_Linear)
		{
			float DT = Keys[NumKeys - 2].Time - Keys[NumKeys - 1].Time;

			if (FMath::IsNearlyZero(DT))
			{
				InterpVal = Keys[NumKeys - 1].Value;
			}
			else
			{
				float DV = Keys[NumKeys - 2].Value - Keys[NumKeys - 1].Value;
				float Slope = DV / DT;

				InterpVal = Slope * (InTime - Keys[NumKeys - 1].Time) + Keys[NumKeys - 1].Value;
			}
		}
		else
		{
			// Otherwise if constant or in a cycle or oscillate, always use the last key value
			InterpVal = Keys[NumKeys - 1].Value;
		}
	}

	return InterpVal + CycleValueOffset;
}


bool FSimpleCurve::operator==(const FSimpleCurve& Curve) const
{
	if(Keys.Num() != Curve.Keys.Num())
	{
		return false;
	}

	for(int32 i = 0;i<Keys.Num();++i)
	{
		if(!(Keys[i] == Curve.Keys[i]))
		{
			return false;
		}
	}

	return true;
}

