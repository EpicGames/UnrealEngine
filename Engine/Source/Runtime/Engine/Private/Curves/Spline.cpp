// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/Spline.h"

#include "Algo/Sort.h"
#include "BoxTypes.h"
#include "Components/SplineComponent.h"	// for FSplineCurves, ideally removed
#include "Curves/Splines/TangentSpline.h"
#include "Misc/Base64.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

DEFINE_LOG_CATEGORY_STATIC(LogSpline, Log, All);

#include UE_INLINE_GENERATED_CPP_BY_NAME(Spline)

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS

namespace UE::Spline
{

int32 GImplementation = 0;
FAutoConsoleVariableRef CVarSplineImplementation(
	TEXT("Spline.Implementation"),
	GImplementation,
	TEXT("0) Not Implemented - 1) Legacy Implementation - 2) New Implementation")
#if !WITH_EDITOR
	,ECVF_ReadOnly		// In non-editor builds, this CVar should exist so that GImplementation is properly set but it should not be changed by the user.
#endif
);

inline UE::Spline::ESplineType GetDefaultSplineType()
{
	switch (GImplementation)
	{
	default:	// fallthrough
	case 0:		return UE::Spline::ESplineType::Unimplemented;
	case 1:		return UE::Spline::ESplineType::LegacyTangent;
	case 2:		return UE::Spline::ESplineType::Tangent;
	}
}

}

using UE::Geometry::FInterval1f;

/** FSpline Implementation */

FSpline::FSpline()
{
	SetCurrentImplementation(UE::Spline::GetDefaultSplineType());

#if WITH_EDITOR
	PreviousImplementation = UE::Spline::ESplineType::Unimplemented;
#endif
}

FSpline::FSpline(UE::Spline::ESplineType InType)
{
	SetCurrentImplementation(InType);

#if WITH_EDITOR
	PreviousImplementation = UE::Spline::ESplineType::Unimplemented;
#endif
}

FSpline::FSpline(const FSpline& Other)
{
	*this = Other;
}

FSpline& FSpline::operator=(const FSpline& Other)
{
	SetCurrentImplementation(Other.CurrentImplementation);

	switch (CurrentImplementation)
	{
	default:
		break;

	case UE::Spline::ESplineType::LegacyTangent:
		*LegacySplineImpl = *Other.LegacySplineImpl;
		break;

	case UE::Spline::ESplineType::Tangent:
		*Get<UE::Spline::ESplineType::Tangent>() = *Other.Get<UE::Spline::ESplineType::Tangent>();
		break;
	}
		
	return *this;
}

FSpline& FSpline::operator=(const FSplineCurves& Other)
{
	SetCurrentImplementation(UE::Spline::ESplineType::Tangent);

	*Get<UE::Spline::ESplineType::Tangent>() = FTangentSpline(Other);
	
	return *this;
}

FVector FSpline::Evaluate(float Param) const
{
	switch (CurrentImplementation)
	{
	default:
		break;

	case UE::Spline::ESplineType::LegacyTangent:
		return LegacySplineImpl->PositionCurve.Eval(Param);

	case UE::Spline::ESplineType::Tangent:
		return Get<UE::Spline::ESplineType::Tangent>()->EvaluatePosition(Param);
	}

	return FVector();
}

FVector FSpline::EvaluateDerivative(float Param) const
{
	switch (CurrentImplementation)
	{
	default:
		break;

	case UE::Spline::ESplineType::LegacyTangent:
		return LegacySplineImpl->PositionCurve.EvalDerivative(Param);

	case UE::Spline::ESplineType::Tangent:
		return Get<UE::Spline::ESplineType::Tangent>()->EvaluateDerivative(Param);
	}

	return FVector();
}

UE::Geometry::FInterval1f FSpline::GetParameterSpace() const
{
	using UE::Geometry::FInterval1f;

	switch (CurrentImplementation)
	{
	default:
		break;

	case UE::Spline::ESplineType::LegacyTangent:
		if (LegacySplineImpl->PositionCurve.Points.Num() > 0)
		{
			return FInterval1f(LegacySplineImpl->PositionCurve.Points[0].InVal, LegacySplineImpl->PositionCurve.Points.Last().InVal);
		}
		break;

	case UE::Spline::ESplineType::Tangent:
		return Get<UE::Spline::ESplineType::Tangent>()->GetParameterSpace();
	}

	return FInterval1f();
}

int32 FSpline::GetNumberOfSegments() const
{
	switch (CurrentImplementation)
	{
	default:
		return 0;

	case UE::Spline::ESplineType::LegacyTangent:
		return LegacySplineImpl->PositionCurve.bIsLooped ? LegacySplineImpl->PositionCurve.Points.Num() : FMath::Max(LegacySplineImpl->PositionCurve.Points.Num() - 1, 0);

	case UE::Spline::ESplineType::Tangent:
		// fallthrough for all ISplineInterfaces
		return SplineImpl->GetNumberOfSegments();
	}
}

void FSpline::Reset()
{
	switch (CurrentImplementation)
	{
	default:
		break;

	case UE::Spline::ESplineType::LegacyTangent:
		LegacySplineImpl->PositionCurve.Reset();
		LegacySplineImpl->PositionCurve.ClearLoopKey();
		LegacySplineImpl->RotationCurve.Reset();
		LegacySplineImpl->RotationCurve.ClearLoopKey();
		LegacySplineImpl->ScaleCurve.Reset();
		LegacySplineImpl->ScaleCurve.ClearLoopKey();
		LegacySplineImpl->ReparamTable.Reset();
		LegacySplineImpl->ReparamTable.ClearLoopKey();
		break;

	case UE::Spline::ESplineType::Tangent:
		// fallthrough for all ISplineInterfaces
		SplineImpl->Clear();
		break;
	}
}

bool FSpline::operator==(const FSpline& Other) const
{
	if (CurrentImplementation != Other.CurrentImplementation)
	{
		return false;
	}

	switch (CurrentImplementation)
	{
	default:
		return false;

	case UE::Spline::ESplineType::LegacyTangent:
		return LegacySplineImpl->PositionCurve == Other.LegacySplineImpl->PositionCurve &&
			LegacySplineImpl->RotationCurve == Other.LegacySplineImpl->RotationCurve &&
			LegacySplineImpl->ScaleCurve == Other.LegacySplineImpl->ScaleCurve;

	case UE::Spline::ESplineType::Tangent:
		// fallthrough for all ISplineInterfaces
		return SplineImpl->IsEqual(Other.SplineImpl.Get());

	}
}

bool FSpline::operator!=(const FSpline& Other) const
{
	return !(*this == Other);
}

bool FSpline::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	
	// Data format:

	// Byte 1 - The data format, determine by CurrentImplementation at the time of last save
	// Remaining N Bytes - Spline data (or empty, if byte 0 is 0). The format is determined by byte 1.
	
	if (Ar.IsLoading())
	{
		SerializeLoad(Ar);
	}
	else
	{
		SerializeSave(Ar);
	}

	return true;
}

void FSpline::SerializeLoad(FArchive& Ar)
{
	uint8 PreviousImpl;
	Ar << PreviousImpl;

#if WITH_EDITOR
	PreviousImplementation = static_cast<UE::Spline::ESplineType>(PreviousImpl);
#endif

	SetCurrentImplementation(static_cast<UE::Spline::ESplineType>(PreviousImpl));

	switch (CurrentImplementation)
	{
	default: break;

	case UE::Spline::ESplineType::LegacyTangent: Ar << *LegacySplineImpl; break;
		
	case UE::Spline::ESplineType::Tangent:
		// fallthrough for all ISplineInterfaces
		Ar << *SplineImpl; break;
	}
}

void FSpline::SerializeSave(FArchive& Ar) const
{
	uint8 CurrentImpl = static_cast<uint8>(CurrentImplementation);
	Ar << CurrentImpl;

	switch (CurrentImplementation)
	{
	default: break;

	case UE::Spline::ESplineType::LegacyTangent: Ar << *LegacySplineImpl; break;

	case UE::Spline::ESplineType::Tangent:
		// fallthrough for all ISplineInterfaces
		Ar << *SplineImpl; break;
	}
}

// todo: make cvar when importing handles it changing.
constexpr bool bEncodeAsHex = true;

bool FSpline::ExportTextItem(FString& ValueStr, FSpline const& DefaultValue, class UObject* Parent, int32 PortFlags, class UObject* ExportRootScope) const
{
	// serialize our spline
    TArray<uint8> SplineWriteBuffer;
    FMemoryWriter MemWriter(SplineWriteBuffer);
    SerializeSave(MemWriter);

	if (bEncodeAsHex)
	{
		FString HexStr = FString::FromHexBlob(SplineWriteBuffer.GetData(), SplineWriteBuffer.Num());
		ValueStr = FString::Printf(TEXT("SplineData SplineDataLen=%d SplineData=%s\r\n"), HexStr.Len(), *HexStr);
	}
	else
	{
		FString Base64String = FBase64::Encode(SplineWriteBuffer);

		// Base64 encoding uses the '/' character, but T3D interprets '//' as some kind of
		// terminator (?). If it occurs then the string passed to ImportTextItem() will
		// come back as full of nullptrs. So we will swap in '-' here, and swap back to '/' in ImportTextItem()
		Base64String.ReplaceCharInline('/', '-');

		ValueStr = FString::Printf(TEXT("SplineData SplineDataLen=%d SplineData=%s\r\n"), Base64String.Len(), *Base64String);
	}
	
	return true;
}

bool FSpline::ImportTextItem(const TCHAR*& SourceText, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	// todo: SplineData= is followed by a byte identifying the implementation type, we should
	// validate this on import and potentially change spline type if it doesn't match our current type

	if (FParse::Command(&SourceText, TEXT("SplineData")))	
	{
		static const TCHAR SplineDataLenToken[] = TEXT("SplineDataLen=");
		const TCHAR* FoundSplineDataLenStart = FCString::Strifind(SourceText, SplineDataLenToken);
		if (FoundSplineDataLenStart)
		{
			SourceText = FoundSplineDataLenStart + FCString::Strlen(SplineDataLenToken);
			int32 SplineDataLen = FCString::Atoi(SourceText);

			static const TCHAR SplineDataToken[] = TEXT("SplineData=");
			const TCHAR* FoundSplineDataStart = FCString::Strifind(SourceText, SplineDataToken);
			if (FoundSplineDataStart)
			{
				SourceText = FoundSplineDataStart + FCString::Strlen(SplineDataToken);
				FString SplineData = FString::ConstructFromPtrSize(SourceText, SplineDataLen);

				// fix-up the hack applied to the Base64-encoded string in ExportTextItem()
				SplineData.ReplaceCharInline('-', '/');

				TArray<uint8> SplineReadBuffer;
				bool bDecoded;
				
				if (bEncodeAsHex)
				{
					SplineReadBuffer.SetNum(SplineDataLen / 2);
					bDecoded = FString::ToHexBlob(SplineData, SplineReadBuffer.GetData(), SplineReadBuffer.Num());
				}
				else
				{
					bDecoded = FBase64::Decode(SplineData, SplineReadBuffer);
				}

				if (bDecoded)
				{
					FMemoryReader MemReader(SplineReadBuffer);
					SerializeLoad(MemReader);
				}
			}
		}
	}
	
	return true;
}

UE::Spline::ESplineType FSpline::GetCurrentImplementation() const
{ 
	return CurrentImplementation;
}

#if WITH_EDITOR
UE::Spline::ESplineType FSpline::GetPreviousImplementation() const
{
	return PreviousImplementation;
}
#endif

UE::Spline::ESplineType FSpline::GetDefaultImplementation()
{
	return UE::Spline::GetDefaultSplineType();
}

void FSpline::SetCurrentImplementation(UE::Spline::ESplineType Implementation)
{
	CurrentImplementation = Implementation;

	LegacySplineImpl.Reset();
	SplineImpl.Reset();

	switch (CurrentImplementation)
	{
	default: break;		// Intentionally doing nothing
	case UE::Spline::ESplineType::LegacyTangent:	LegacySplineImpl = MakeUnique<FLegacyTangentSpline>(); break;
	case UE::Spline::ESplineType::Tangent:			SplineImpl = MakeUnique<FTangentSpline>(); break;
	}
}

PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS