// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/Splines/TangentSpline.h"

#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Misc/ScopeRWLock.h"
#include "Components/SplineComponent.h"

using UE::Geometry::FInterval1f;

DEFINE_LOG_CATEGORY_STATIC(LogTangentSpline, Log, All);

namespace UE::Spline
{

int32 GParameterizationPolicy = 0;
FAutoConsoleVariableRef CVarParameterizationPolicy(
	TEXT("TangentSpline.ParameterizationPolicy"),
	GParameterizationPolicy,
	TEXT("0) Uniform - 1) Chord Length - 2) Centripetal")
);

bool GUseLegacyPositionEvaluation = false;
FAutoConsoleVariableRef CVarUseLegacyPositionEvaluation(
	TEXT("TangentSpline.UseLegacyPositionEvaluation"),
	GUseLegacyPositionEvaluation,
	TEXT("If true, evaluating the position channel always routes through an interp curve.")
);

bool GUseLegacyRotationEvaluation = false;
FAutoConsoleVariableRef CVarUseLegacyRotationEvaluation(
	TEXT("TangentSpline.UseLegacyRotationEvaluation"),
	GUseLegacyRotationEvaluation,
	TEXT("If true, evaluating the rotation channel always routes through an interp curve.")
);

bool GUseLegacyScaleEvaluation = false;
FAutoConsoleVariableRef CVarUseLegacyScaleEvaluation(
	TEXT("TangentSpline.UseLegacyScaleEvaluation"),
	GUseLegacyScaleEvaluation,
	TEXT("If true, evaluating the scale channel always routes through an interp curve.")
);

bool GImmediatelyUpdateLegacyCurves = false;
FAutoConsoleVariableRef CVarImmediatelyUpdateLegacyCurves(
	TEXT("TangentSpline.ImmediatelyUpdateLegacyCurves"),
	GImmediatelyUpdateLegacyCurves,
	TEXT("If true, mutating operations immediately rebuild legacy curves. If false, legacy curves are updated only when requested.")
);

bool GValidateRotScale = false;
FAutoConsoleVariableRef CVarValidateRotScale(
	TEXT("TangentSpline.ValidateRotScale"),
	GValidateRotScale,
	TEXT("True if we should validate rotation and scale attributes when structurally modifying the spline.")
);

bool GSkipDoubleUpdate = true;
FAutoConsoleVariableRef CVarSkipDoubleUpdate(
	TEXT("TangentSpline.SkipDoubleUpdate"),
	GSkipDoubleUpdate,
	TEXT("When true, skip updating the spline twice if UpdateSpline with parameters is called.")
);

}

inline UE::Geometry::Spline::EParameterizationPolicy GetParameterizationPolicy()
{
	switch (UE::Spline::GParameterizationPolicy)
	{
	default:	// fallthrough
	case 0: return UE::Geometry::Spline::EParameterizationPolicy::Uniform;
	case 1: return UE::Geometry::Spline::EParameterizationPolicy::ChordLength;
	case 2: return UE::Geometry::Spline::EParameterizationPolicy::Centripetal;
	}
}

/**
 * Converts from EInterpCurveMode to ETangentMode
 */
inline UE::Geometry::Spline::ETangentMode ConvertInterpCurveModeToTangentMode(EInterpCurveMode Mode)
{
	switch (Mode)
	{
	case CIM_Linear:
		return UE::Geometry::Spline::ETangentMode::LegacyLinear; // Default to legacy for compatibility
	case CIM_CurveAuto:
		return UE::Geometry::Spline::ETangentMode::LegacyAuto; // Default to legacy for compatibility
	case CIM_Constant:
		return UE::Geometry::Spline::ETangentMode::Constant;
	case CIM_CurveUser:
		return UE::Geometry::Spline::ETangentMode::User;
	case CIM_CurveBreak:
		return UE::Geometry::Spline::ETangentMode::Broken;
	case CIM_CurveAutoClamped:
		return UE::Geometry::Spline::ETangentMode::LegacyAutoClamped; // Default to legacy for compatibility
	case CIM_Unknown:
	default:
		return UE::Geometry::Spline::ETangentMode::Unknown;
	}
}

/**
 * Converts from ETangentMode to EInterpCurveMode
 */
inline EInterpCurveMode ConvertTangentModeToInterpCurveMode(UE::Geometry::Spline::ETangentMode Mode)
{
	switch (Mode)
	{
	case UE::Geometry::Spline::ETangentMode::LegacyLinear:
	case UE::Geometry::Spline::ETangentMode::Linear:
		return CIM_Linear;
	case UE::Geometry::Spline::ETangentMode::Auto:
	case UE::Geometry::Spline::ETangentMode::LegacyAuto:
		return CIM_CurveAuto;
	case UE::Geometry::Spline::ETangentMode::Constant:
		return CIM_Constant;
	case UE::Geometry::Spline::ETangentMode::User:
		return CIM_CurveUser;
	case UE::Geometry::Spline::ETangentMode::Broken:
		return CIM_CurveBreak;
	case UE::Geometry::Spline::ETangentMode::AutoClamped:
	case UE::Geometry::Spline::ETangentMode::LegacyAutoClamped:
		return CIM_CurveAutoClamped;
	case UE::Geometry::Spline::ETangentMode::Unknown:
	default:
		return CIM_Unknown;
	}
}

FTangentSpline::FTangentSpline()
{
	// This is a hack. We just need to make sure that this spline type registers itself with the spline registry.
	// This should be removed in the future once we can register this spline type automatically without instantiation.
	UE::Geometry::Spline::TTangentBezierSpline<float> AutoRegister;

	CreateAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FQuat>>(RotationAttrName);
	CreateAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FVector>>(ScaleAttrName);

	ValidateRotScale();
}

FTangentSpline::FTangentSpline(const FTangentSpline& Other)
{
	*this = Other;

	ValidateRotScale();
}

FTangentSpline::FTangentSpline(const FLegacyTangentSpline& Other)
{
	CreateAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FQuat>>(RotationAttrName);
	CreateAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FVector>>(ScaleAttrName);

	TArray<FTangentBezierControlPoint> Points;
	TArray<float> Parameters;
	for (int32 Idx = 0; Idx < Other.PositionCurve.Points.Num(); Idx++)
	{
		const FInterpCurvePoint<FVector>& Position = Other.PositionCurve.Points[Idx];

		FTangentBezierControlPoint& Point = Points.AddDefaulted_GetRef();
		Point.Position = Position.OutVal;
		Point.TangentIn = Position.ArriveTangent;
		Point.TangentOut = Position.LeaveTangent;
		Point.TangentMode = ConvertInterpCurveModeToTangentMode(Position.InterpMode);

		Parameters.Add(Position.InVal);
	}

	// initialize spline
	GetSpline().SetPoints(Points, UE::Geometry::Spline::EParameterizationPolicy::Uniform);
	SetParameters(Parameters);

	ResetRotation();
	ResetScale();
	for (int32 Idx = 0; Idx < Other.PositionCurve.Points.Num(); Idx++)
	{
		if (Other.RotationCurve.Points.IsValidIndex(Idx))
		{
			SetAttributeValue<FQuat>(RotationAttrName, Other.RotationCurve.Points[Idx].OutVal, Idx);
		}

		if (Other.ScaleCurve.Points.IsValidIndex(Idx))
		{
			SetAttributeValue<FVector>(ScaleAttrName, Other.ScaleCurve.Points[Idx].OutVal, Idx);
		}
	}
	ValidateRotScale();

	FUpdateSplineParams UpdateParams;
	UpdateParams.bClosedLoop = Other.PositionCurve.bIsLooped;
	// If the source curves encode an override loop offset parameter, convert it to an absolute loop position
	// The source curve is assumed to encode an override loop offset parameter if it
	// 1) is closed,
	// 2) has at least 1 point,
	// 3) specifies a positive offset that is not exactly equal to 1.
	if (Other.PositionCurve.bIsLooped && Other.PositionCurve.Points.Num() > 0 && Other.PositionCurve.LoopKeyOffset > 0.0f && Other.PositionCurve.LoopKeyOffset != 1.f)
	{
		UpdateParams.bLoopPositionOverride = true;
		UpdateParams.LoopPosition = Other.PositionCurve.Points.Last().InVal + Other.PositionCurve.LoopKeyOffset;
	}

	UpdateSpline(UpdateParams);
}

FTangentSpline::FTangentSpline(const FSplineCurves& Other)
{
	CreateAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FQuat>>(RotationAttrName);
	CreateAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FVector>>(ScaleAttrName);

	TArray<FTangentBezierControlPoint> Points;
	TArray<float> Parameters;
	for (int32 Idx = 0; Idx < Other.Position.Points.Num(); Idx++)
	{
		const FInterpCurvePoint<FVector>& Position = Other.Position.Points[Idx];

		FTangentBezierControlPoint& Point = Points.AddDefaulted_GetRef();
		Point.Position = Position.OutVal;
		Point.TangentIn = Position.ArriveTangent;
		Point.TangentOut = Position.LeaveTangent;
		Point.TangentMode = ConvertInterpCurveModeToTangentMode(Position.InterpMode);

		Parameters.Add(Position.InVal);
	}

	// initialize spline
	GetSpline().SetPoints(Points, UE::Geometry::Spline::EParameterizationPolicy::Uniform);
	SetParameters(Parameters);

	ResetRotation();
	ResetScale();
	for (int32 Idx = 0; Idx < Other.Position.Points.Num(); Idx++)
	{
		FQuat Rotation = FQuat::Identity;
		if (Other.Rotation.Points.IsValidIndex(Idx))
		{
			Rotation = Other.Rotation.Points[Idx].OutVal;
		}
		else
		{
			UE_LOGF(LogTangentSpline, Warning, "Number of rotation values (%d) different than the number of position values (%d)!", Other.Rotation.Points.Num(), Other.Position.Points.Num());
		}

		FVector Scale = FVector::OneVector;
		if (Other.Scale.Points.IsValidIndex(Idx))
		{
			Scale = Other.Scale.Points[Idx].OutVal;
		}
		else
		{
			UE_LOGF(LogTangentSpline, Warning, "Number of scale values (%d) different than the number of position values (%d)!", Other.Scale.Points.Num(), Other.Position.Points.Num());
		}

		SetAttributeValue<FQuat>(RotationAttrName, Rotation, Idx);
		SetAttributeValue<FVector>(ScaleAttrName, Scale, Idx);
	}
	ValidateRotScale();

	FUpdateSplineParams UpdateParams;
	UpdateParams.bClosedLoop = Other.Position.bIsLooped;
	// If the source curves encode an override loop offset parameter, convert it to an absolute loop position
	// The source curve is assumed to encode an override loop offset parameter if it
	// 1) is closed,
	// 2) has at least 1 point,
	// 3) specifies a positive offset that is not exactly equal to 1.
	if (Other.Position.bIsLooped && Other.Position.Points.Num() > 0 && Other.Position.LoopKeyOffset > 0.0f && Other.Position.LoopKeyOffset != 1.f)
	{
		UpdateParams.bLoopPositionOverride = true;
		UpdateParams.LoopPosition = Other.Position.Points.Last().InVal + Other.Position.LoopKeyOffset;
	}
	
	UpdateSpline(UpdateParams);

	Version = Other.Version;
}

/** Copy assignment */
FTangentSpline& FTangentSpline::operator=(const FTangentSpline& Other)
{
	if (this != &Other)
	{
		TMultiSpline::operator=(Other);
		PositionCurve = Other.PositionCurve;
		RotationCurve = Other.RotationCurve;
		ScaleCurve = Other.ScaleCurve;
		ReparamStepsPerSegment = Other.ReparamStepsPerSegment;
		Version = Other.Version;
	}

	return *this;
}

bool FTangentSpline::operator!=(const FTangentSpline& Other) const
{
	return !(*this == Other);
}

bool FTangentSpline::IsNearlyEqual(const FTangentSpline& Other, const float RelativeTolerance, const float AbsoluteTolerance) const
{
	const int32 NumPoints = GetNumberOfPoints();
	if (Other.GetNumberOfPoints() != NumPoints)
	{
		return false;
	}

	for (int32 PointIdx = 0; PointIdx < NumPoints; ++PointIdx)
	{
		const FVector Location = GetLocation(PointIdx);
		const FVector OtherLocation = Other.GetLocation(PointIdx);
		const float LocationMagDelta = (Location - OtherLocation).Length();
		const float LocationMagTolerance = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(Location.Length(), OtherLocation.Length()));
		if (LocationMagDelta > LocationMagTolerance)
		{
			return false;
		}

		const FVector InTangent = GetInTangent(PointIdx);
		const FVector OtherInTangent = Other.GetInTangent(PointIdx);
		const float InTangentDelta = (InTangent - OtherInTangent).Length();
		const float InTangentTolerance = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(InTangent.Length(), OtherInTangent.Length()));
		if (InTangentDelta > InTangentTolerance)
		{
			return false;
		}

		const FVector OutTangent = GetOutTangent(PointIdx);
		const FVector OtherOutTangent = Other.GetOutTangent(PointIdx);
		const float OutTangentDelta = (OutTangent - OtherOutTangent).Length();
		const float OutTangentTolerance = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(OutTangent.Length(), OtherOutTangent.Length()));
		if (OutTangentDelta > OutTangentTolerance)
		{
			return false;
		}

		const FQuat Rotation = GetRotation(PointIdx);
		const FQuat OtherRotation = Other.GetRotation(PointIdx);
		const FQuat OtherRotationNeg = FQuat(-OtherRotation.X, -OtherRotation.Y, -OtherRotation.Z, -OtherRotation.W);

		auto QuatComponentsNearlyEqual = [AbsoluteTolerance, RelativeTolerance](const FQuat& A, const FQuat& B)
			{
				const float DX = FMath::Abs(A.X - B.X);
				const float TX = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(FMath::Abs(A.X), FMath::Abs(B.X)));
				if (DX > TX) return false;

				const float DY = FMath::Abs(A.Y - B.Y);
				const float TY = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(FMath::Abs(A.Y), FMath::Abs(B.Y)));
				if (DY > TY) return false;

				const float DZ = FMath::Abs(A.Z - B.Z);
				const float TZ = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(FMath::Abs(A.Z), FMath::Abs(B.Z)));
				if (DZ > TZ) return false;

				const float DW = FMath::Abs(A.W - B.W);
				const float TW = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(FMath::Abs(A.W), FMath::Abs(B.W)));
				return DW <= TW;
			};

		if (!QuatComponentsNearlyEqual(Rotation, OtherRotation) && !QuatComponentsNearlyEqual(Rotation, OtherRotationNeg))
		{
			return false;
		}

		const FVector Scale = GetScale(PointIdx);
		const FVector OtherScale = Other.GetScale(PointIdx);
		const float ScaleMagDelta = (Scale - OtherScale).Length();
		const float ScaleMagTolerance = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(Scale.Length(), OtherScale.Length()));
		if (ScaleMagDelta > ScaleMagTolerance)
		{
			return false;
		}

		const float Parameter = GetParameterAtIndex(PointIdx);
		const float OtherParameter = Other.GetParameterAtIndex(PointIdx);
		const float ParameterDelta = FMath::Abs(Parameter - OtherParameter);
		const float ParameterTolerance = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(FMath::Abs(Parameter), FMath::Abs(OtherParameter)));
		if (ParameterDelta > ParameterTolerance)
		{
			return false;
		}
	}

	const TArray<FName> OtherFloatChannelNames = Other.GetAttributeChannelNamesByValueType<float>();
	for (const FName& FloatChannelName : GetAttributeChannelNamesByValueType<float>())
	{
		if (!OtherFloatChannelNames.Contains(FloatChannelName))
		{
			return false;
		}

		const int32 NumFloatChannelPoints = NumAttributeValues<float>(FloatChannelName);
		if (Other.NumAttributeValues<float>(FloatChannelName) != NumFloatChannelPoints)
		{
			return false;
		}

		for (int32 FloatChannelIdx = 0; FloatChannelIdx < NumFloatChannelPoints; ++FloatChannelIdx)
		{
			const float FloatChannelValue = GetAttributeValue<float>(FloatChannelName, FloatChannelIdx);
			const float OtherFloatChannelValue = Other.GetAttributeValue<float>(FloatChannelName, FloatChannelIdx);
			const float FloatChannelDelta = FMath::Abs(FloatChannelValue - OtherFloatChannelValue);
			const float FloatChannelTolerance = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(FMath::Abs(FloatChannelValue), FMath::Abs(OtherFloatChannelValue)));
			if (FloatChannelDelta > FloatChannelTolerance)
			{
				return false;
			}

			const float FloatChannelParameter = GetAttributeParameter<float>(FloatChannelName, FloatChannelIdx);
			const float OtherFloatChannelParameter = Other.GetAttributeParameter<float>(FloatChannelName, FloatChannelIdx);
			const float ParameterDelta = FMath::Abs(FloatChannelParameter - OtherFloatChannelParameter);
			const float ParameterTolerance = FMath::Max(AbsoluteTolerance, RelativeTolerance * FMath::Max(FMath::Abs(FloatChannelParameter), FMath::Abs(OtherFloatChannelParameter)));
			if (ParameterDelta > ParameterTolerance)
			{
				return false;
			}
		}
	}

	return true;
}

bool FTangentSpline::Serialize(FArchive& Ar)
{
	TMultiSpline::Serialize(Ar);
	Ar << ReparamStepsPerSegment;

	if (Ar.IsLoading())
	{
		// Do not serialize legacy curves, only generate if loading.
		RebuildLegacyCurves();
	}

	return true;
}

float FTangentSpline::GetSegmentLength(const int32 Index, const float Param, const FVector& Scale3D) const
{
	const int32 NumPoints = GetNumberOfPoints();
	const int32 LastPoint = NumPoints - 1;

	check(Index >= 0 && ((IsClosedLoop() && Index < NumPoints) || (!IsClosedLoop() && Index < LastPoint)));
	check(Param >= 0.0f && Param <= 1.0f);

	// Evaluate the length of a Hermite spline segment.
	// This calculates the integral of |dP/dt| dt, where P(t) is the spline equation with components (x(t), y(t), z(t)).
	// This isn't solvable analytically, so we use a numerical method (Legendre-Gauss quadrature) which performs very well
	// with functions of this type, even with very few samples.  In this case, just 5 samples is sufficient to yield a
	// reasonable result.

	struct FLegendreGaussCoefficient
	{
		float Abscissa;
		float Weight;
	};

	static const FLegendreGaussCoefficient LegendreGaussCoefficients[] =
	{
		{ 0.0f, 0.5688889f },
		{ -0.5384693f, 0.47862867f },
		{ 0.5384693f, 0.47862867f },
		{ -0.90617985f, 0.23692688f },
		{ 0.90617985f, 0.23692688f }
	};

	const auto& P0 = GetSpline().GetValue(Index);
	const auto& T0 = GetSpline().GetTangentOut(Index);
	const auto& P1 = GetSpline().GetValue(Index == LastPoint ? 0 : Index + 1);
	const auto& T1 = GetSpline().GetTangentIn(Index == LastPoint ? 0 : Index + 1);

	// Special cases for linear
	if (GetSpline().GetTangentModes()[Index] == UE::Geometry::Spline::ETangentMode::Linear ||
		GetSpline().GetTangentModes()[Index] == UE::Geometry::Spline::ETangentMode::LegacyLinear)
	{
		return ((P1 - P0) * Scale3D).Size() * Param;
	}

	// Cache the coefficients to be fed into the function to calculate the spline derivative at each sample point as they are constant.
	const FVector Coeff1 = ((P0 - P1) * 2.0f + T0 + T1) * 3.0f;
	const FVector Coeff2 = (P1 - P0) * 6.0f - T0 * 4.0f - T1 * 2.0f;
	const FVector Coeff3 = T0;

	const float HalfParam = Param * 0.5f;

	float Length = 0.0f;
	for (const auto& LegendreGaussCoefficient : LegendreGaussCoefficients)
	{
		// Calculate derivative at each Legendre-Gauss sample, and perform a weighted sum
		const float Alpha = HalfParam * (1.0f + LegendreGaussCoefficient.Abscissa);
		const FVector Derivative = ((Coeff1 * Alpha + Coeff2) * Alpha + Coeff3) * Scale3D;
		Length += Derivative.Size() * LegendreGaussCoefficient.Weight;
	}
	Length *= HalfParam;

	return Length;
}

float FTangentSpline::GetSplineLength() const
{
	// Evaluate the ParamToLength channel at the very end of the spline
	const float MaxParameter = GetParameterSpace().Max;
	return GetDistanceAtParameter(MaxParameter);
}

void FTangentSpline::SetClosedLoop(bool bInClosedLoop)
{
	SetClosedLoop(bInClosedLoop, true);
}

void FTangentSpline::SetClosedLoop(bool bInClosedLoop, bool bUpdateSpline)
{
	if (bInClosedLoop == GetSpline().IsClosedLoop())
	{
		return;
	}
	TRACE_CPUPROFILER_EVENT_SCOPE(FTangentSpline::SetClosedLoop);

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();

	GetSpline().SetClosedLoop(bInClosedLoop);

	if (UE::Geometry::Spline::TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FVector>>(ScaleAttrName))
	{
		ScaleChild->SetClosedLoop(bInClosedLoop);
	}

	if (UE::Geometry::Spline::TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FQuat>>(RotationAttrName))
	{
		RotChild->SetClosedLoop(bInClosedLoop);
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}
}

void FTangentSpline::Reset()
{
	Clear();
	GetSpline().SetTangentModes(TArray<UE::Geometry::Spline::ETangentMode>());
	PositionCurve.Reset();
	RotationCurve.Reset();
	ScaleCurve.Reset();

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

void FTangentSpline::ResetRotation()
{
	ClearAttributeChannel(RotationAttrName);

	for (int32 Idx = 0; Idx < GetNumberOfPoints(); Idx++)
	{
		AddAttributeValue<FQuat>(RotationAttrName, FQuat::Identity, static_cast<float>(Idx));
	}

	SetAttributeChannelRange(RotationAttrName, FInterval1f(0.f, 1.f), EMappingRangeSpace::Normalized);
	if (UE::Geometry::Spline::TTangentBezierSpline<FQuat>* RotSpline = GetTypedAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FQuat>>(RotationAttrName))
	{
		RotSpline->SetKnotVector(GetSpline().GetKnotVector());
	}

	MarkLegacyCurvesDirty();
}

void FTangentSpline::ResetScale()
{
	ClearAttributeChannel(ScaleAttrName);

	for (int32 Idx = 0; Idx < GetNumberOfPoints(); Idx++)
	{
		AddAttributeValue<FVector>(ScaleAttrName, FVector::OneVector, static_cast<float>(Idx));
	}

	SetAttributeChannelRange(ScaleAttrName, FInterval1f(0.f, 1.f), EMappingRangeSpace::Normalized);
	if (UE::Geometry::Spline::TTangentBezierSpline<FVector>* ScaleSpline = GetTypedAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FVector>>(ScaleAttrName))
	{
		ScaleSpline->SetKnotVector(GetSpline().GetKnotVector());
	}

	MarkLegacyCurvesDirty();
}

void FTangentSpline::AddPoint(const FSplinePoint& Point)
{
	using namespace UE::Geometry::Spline;

	ValidateRotScale();
	ON_SCOPE_EXIT
	{
		SetAttributeChannelRange(ScaleAttrName, FInterval1f(0.f, 1.f), EMappingRangeSpace::Normalized);
		SetAttributeChannelRange(RotationAttrName, FInterval1f(0.f, 1.f), EMappingRangeSpace::Normalized);
		ValidateRotScale();
	};

	FTangentBezierControlPoint ControlPoint = ConvertToTangentBezierControlPoint(Point);

	const FInterval1f ParameterSpace = GetParameterSpace();
	const float& Parameter = Point.InputKey;

	if (ParameterSpace.IsEmpty() || Parameter > ParameterSpace.Max)
	{
		/* Append Case */

		GetSpline().AppendPoint(ControlPoint, Parameter);
		const TArray<FKnot> NewKnotVector = GetSpline().GetKnotVector();

		// We will append points to the scale and rotation splines now.
		// We have to append with a policy because Parameter is not defined w.r.t. the child spline spaces.
		// We cannot map it into the child space because the mapping functions do not extrapolate outside of the mapped to space.
		// This is fine because we immediately propagate the primary knot vector which overrides whatever parameter is generated by the policy.

		if (TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
		{
			ScaleChild->AppendPoint(TTangentBezierSpline<FVector>::FTangentBezierControlPoint(Point.Scale), EParameterizationPolicy::Uniform);
			ScaleChild->SetKnotVector(NewKnotVector);
		}

		if (TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
		{
			RotChild->AppendPoint(TTangentBezierSpline<FQuat>::FTangentBezierControlPoint(Point.Rotation.Quaternion()), EParameterizationPolicy::Uniform);
			RotChild->SetKnotVector(NewKnotVector);
		}
	}
	else if (Parameter < ParameterSpace.Min)
	{
		/* Prepend Case */

		GetSpline().PrependPoint(ControlPoint, Parameter);
		const TArray<FKnot> NewKnotVector = GetSpline().GetKnotVector();

		// We will prepend points to the scale and rotation splines now.
		// We have to prepend with a policy because Parameter is not defined w.r.t. the child spline spaces.
		// We cannot map it into the child space because the mapping functions do not extrapolate outside of the mapped to space.
		// This is fine because we immediately propagate the primary knot vector which overrides whatever parameter is generated by the policy.

		if (TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
		{
			ScaleChild->PrependPoint(TTangentBezierSpline<FVector>::FTangentBezierControlPoint(Point.Scale), EParameterizationPolicy::Uniform);
			ScaleChild->SetKnotVector(NewKnotVector);
		}

		if (TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
		{
			RotChild->PrependPoint(TTangentBezierSpline<FQuat>::FTangentBezierControlPoint(Point.Rotation.Quaternion()), EParameterizationPolicy::Uniform);
			RotChild->SetKnotVector(NewKnotVector);
		}
	}
	else
	{
		/* Insert Case */

		int32 InsertedIndex = GetSpline().InsertPoint(ConvertToTangentBezierControlPoint(Point), Parameter);
		const TArray<FKnot> NewKnotVector = GetSpline().GetKnotVector();

		if (TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
		{
			float ScaleParam = MapParameterToChildSpace(ScaleAttrName, Parameter);
			ScaleChild->InsertPoint(TTangentBezierSpline<FVector>::FTangentBezierControlPoint(Point.Scale), ScaleParam);
			ScaleChild->SetKnotVector(NewKnotVector);
		}

		if (TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
		{
			float RotParam = MapParameterToChildSpace(RotationAttrName, Parameter);
			RotChild->InsertPoint(TTangentBezierSpline<FQuat>::FTangentBezierControlPoint(Point.Rotation.Quaternion()), RotParam);
			RotChild->SetKnotVector(NewKnotVector);
		}
	}

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

void FTangentSpline::InsertPoint(const FSplinePoint& Point, int32 Index)
{
	using namespace UE::Geometry::Spline;

	const int32 NumPoints = GetNumberOfPoints();

	// We allow Index == NumPoints for appending.
	if (Index < 0 || Index > NumPoints)
	{
		return;
	}

	ValidateRotScale();
	ON_SCOPE_EXIT
	{
		SetAttributeChannelRange(ScaleAttrName, FInterval1f(0.f, 1.f), EMappingRangeSpace::Normalized);
		SetAttributeChannelRange(RotationAttrName, FInterval1f(0.f, 1.f), EMappingRangeSpace::Normalized);
		ValidateRotScale();
	};

	FTangentBezierControlPoint ControlPoint = ConvertToTangentBezierControlPoint(Point);

	if (NumPoints == 0 || Index == NumPoints)
	{
		/* Append Case */

		GetSpline().AppendPoint(ControlPoint, EParameterizationPolicy::Uniform);
		const TArray<FKnot> NewKnotVector = GetSpline().GetKnotVector();

		if (TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
		{
			ScaleChild->AppendPoint(TTangentBezierSpline<FVector>::FTangentBezierControlPoint(Point.Scale), EParameterizationPolicy::Uniform);
			ScaleChild->SetKnotVector(NewKnotVector);
		}

		if (TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
		{
			RotChild->AppendPoint(TTangentBezierSpline<FQuat>::FTangentBezierControlPoint(Point.Rotation.Quaternion()), EParameterizationPolicy::Uniform);
			RotChild->SetKnotVector(NewKnotVector);
		}
	}
	else if (Index == 0)
	{
		/* Prepend Case */

		GetSpline().PrependPoint(ControlPoint, EParameterizationPolicy::Uniform);
		const TArray<FKnot> NewKnotVector = GetSpline().GetKnotVector();

		if (TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
		{
			ScaleChild->PrependPoint(TTangentBezierSpline<FVector>::FTangentBezierControlPoint(Point.Scale), EParameterizationPolicy::Uniform);
			ScaleChild->SetKnotVector(NewKnotVector);
		}

		if (TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
		{
			RotChild->PrependPoint(TTangentBezierSpline<FQuat>::FTangentBezierControlPoint(Point.Rotation.Quaternion()), EParameterizationPolicy::Uniform);
			RotChild->SetKnotVector(NewKnotVector);
		}
	}
	else
	{
		/* Insert Case */

		// We want to assume Index is in the range [1, NumPoints - 1].
		// Note: We can assume NumPoints >= 2 due to the above prepend/append special cases in addition to the normal bounds check on Index.
		if (!ensureAlways(Index >= 1 && Index <= NumPoints - 1))
		{
			return;
		}

		// We will split the segment parameter range as opposed to finding the parameter of the nearest point on the spline
		// so that insertion doesn't accidentally & prematurely cause a degenerate segment parameter range.
		const int32 SplitSegmentIndex = Index - 1;
		const float Parameter = GetSpline().GetSegmentParameterRange(SplitSegmentIndex).Interpolate(0.5f);

		int32 InsertedIndex = GetSpline().InsertPoint(ControlPoint, Parameter);
		const TArray<FKnot> NewKnotVector = GetSpline().GetKnotVector();

		// This would be a problem...
		ensureAlways(Index == InsertedIndex);

		if (TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
		{
			float ScaleParam = MapParameterToChildSpace(ScaleAttrName, Parameter);
			ScaleChild->InsertPoint(TTangentBezierSpline<FVector>::FTangentBezierControlPoint(Point.Scale), ScaleParam);
			ScaleChild->SetKnotVector(NewKnotVector);
		}

		if (TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
		{
			float RotParam = MapParameterToChildSpace(RotationAttrName, Parameter);
			RotChild->InsertPoint(TTangentBezierSpline<FQuat>::FTangentBezierControlPoint(Point.Rotation.Quaternion()), RotParam);
			RotChild->SetKnotVector(NewKnotVector);
		}
	}

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

FSplinePoint FTangentSpline::GetPoint(int32 Index) const
{
	FSplinePoint Point;

	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return Point;
	}

	Point.InputKey = GetParameterAtIndex(Index);
	Point.Position = GetSpline().GetValue(Index);
	Point.ArriveTangent = GetSpline().GetTangentIn(Index);
	Point.LeaveTangent = GetSpline().GetTangentOut(Index);
	Point.Rotation = GetAttributeValue<FQuat>(RotationAttrName, Index).Rotator();
	Point.Scale = GetAttributeValue<FVector>(ScaleAttrName, Index);
	EInterpCurveMode InterpCurveMode = ConvertTangentModeToInterpCurveMode(GetSpline().GetTangentModes()[Index]);
	Point.Type = ConvertInterpCurveModeToSplinePointType(InterpCurveMode);

	return Point;
}

/** Removes a point from the spline */
void FTangentSpline::RemovePoint(int32 Index)
{
	using namespace UE::Geometry::Spline;

	ValidateRotScale();
	ON_SCOPE_EXIT
	{
		SetAttributeChannelRange(ScaleAttrName, FInterval1f(0.f, 1.f), EMappingRangeSpace::Normalized);
		SetAttributeChannelRange(RotationAttrName, FInterval1f(0.f, 1.f), EMappingRangeSpace::Normalized);

		ValidateRotScale();
	};

	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return;
	}

	GetSpline().RemovePoint(Index);
	const TArray<FKnot> NewKnotVector = GetSpline().GetKnotVector();

	if (TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
	{
		ScaleChild->RemovePoint(Index);
		ScaleChild->SetKnotVector(NewKnotVector);
	}

	if (TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
	{
		RotChild->RemovePoint(Index);
		RotChild->SetKnotVector(NewKnotVector);
	}

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

int32 FTangentSpline::GetNumberOfPoints() const
{
	return GetSpline().GetNumPoints();
}

void FTangentSpline::SetLocation(int32 Index, const FVector& InLocation)
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return;
	}

	GetSpline().ModifyPoint(Index, { InLocation, GetSpline().GetTangentIn(Index), GetSpline().GetTangentOut(Index), GetSpline().GetTangentMode(Index) });

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

FVector FTangentSpline::GetLocation(const int32 Index) const
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return FVector();
	}

	return GetSpline().GetValue(Index);
}

void FTangentSpline::SetInTangent(const int32 Index, const FVector& InTangent)
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return;
	}

	GetSpline().SetPointTangentMode(Index, UE::Geometry::Spline::ETangentMode::User);
	GetSpline().SetTangentIn(Index, InTangent);

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

FVector FTangentSpline::GetInTangent(const int32 Index) const
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return FVector();
	}

	return GetSpline().GetTangentIn(Index);
}

void FTangentSpline::SetOutTangent(const int32 Index, const FVector& OutTangent)
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return;
	}

	GetSpline().SetPointTangentMode(Index, UE::Geometry::Spline::ETangentMode::User);
	GetSpline().SetTangentOut(Index, OutTangent);

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

FVector FTangentSpline::GetOutTangent(const int32 Index) const
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return FVector();
	}

	return GetSpline().GetTangentOut(Index);
}

void FTangentSpline::SetRotation(int32 Index, const FQuat& InRotation)
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return;
	}

	ValidateRotScale();

	SetAttributeValue<FQuat>(RotationAttrName, InRotation, Index);

	MarkLegacyCurvesDirty();
}

FQuat FTangentSpline::GetRotation(const int32 Index) const
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return FQuat::Identity;
	}

	return GetAttributeValue<FQuat>(RotationAttrName, Index);
}

void FTangentSpline::SetScale(int32 Index, const FVector& InScale)
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return;
	}

	ValidateRotScale();

	SetAttributeValue<FVector>(ScaleAttrName, InScale, Index);

	MarkLegacyCurvesDirty();
}

FVector FTangentSpline::GetScale(const int32 Index) const
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return FVector::OneVector;
	}

	return GetAttributeValue<FVector>(ScaleAttrName, Index);
}

void FTangentSpline::SetSplinePointType(int32 Index, EInterpCurveMode Type)
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return;
	}

	GetSpline().SetPointTangentMode(Index, ConvertInterpCurveModeToTangentMode(Type));

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

EInterpCurveMode FTangentSpline::GetSplinePointType(int32 Index) const
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return CIM_Unknown;
	}

	return ConvertTangentModeToInterpCurveMode(GetSpline().GetTangentMode(Index));
}

void FTangentSpline::Reparameterize()
{
	Reparameterize(GetParameterizationPolicy());
}

void FTangentSpline::Reparameterize(UE::Geometry::Spline::EParameterizationPolicy ParameterizationPolicy)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTangentSpline::Reparameterize);

	using namespace UE::Geometry::Spline;

	// todo: Reparameterize attribute channels such that their points remain positioned at the same point in the parameter space of its associated segment.

	GetSpline().Reparameterize(ParameterizationPolicy);
	GetSpline().UpdateTangents();

	const TArray<FKnot> NewKnotVector = GetSpline().GetKnotVector();

	if (TTangentBezierSpline<FVector>* ScaleSpline = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
	{
		ScaleSpline->SetKnotVector(NewKnotVector);
		ScaleSpline->UpdateTangents();
	}

	if (TTangentBezierSpline<FQuat>* RotSpline = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
	{
		RotSpline->SetKnotVector(NewKnotVector);
		RotSpline->UpdateTangents();
	}

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

bool FTangentSpline::SetParameterAtIndex(int32 Index, float Parameter)
{
	const int32 NumPoints = GetNumberOfPoints();

	if (Index < 0 || Index >= NumPoints)
	{
		return false;
	}

	auto ExecuteSetParameter = [this, Index, Parameter]()
	{
		GetSpline().SetParameter(Index, Parameter);

		if (UE::Geometry::Spline::TTangentBezierSpline<FQuat>* RotSpline = GetTypedAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FQuat>>(RotationAttrName))
		{
			RotSpline->SetParameter(Index, Parameter);
		}

		if (UE::Geometry::Spline::TTangentBezierSpline<FVector>* ScaleSpline = GetTypedAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FVector>>(ScaleAttrName))
		{
			ScaleSpline->SetParameter(Index, Parameter);
		}
	};

	const float CurrentPointParameter = GetParameterAtIndex(Index);

	// We do this here so we can check < and > later without technically missing this case.
	if (Parameter == CurrentPointParameter)
	{
		return true;
	}

	// Early out for this trivial case, no reordering is possible and no closing knot fixup is necessary.
	if (NumPoints == 1)
	{
		ExecuteSetParameter();
		return true;
	}

	const int32 PreviousIndex = Index > 0 ? Index - 1 : INDEX_NONE;
	if (PreviousIndex != INDEX_NONE)
	{
		const float PreviousPointParameter = GetParameterAtIndex(PreviousIndex);

		// Detect possible segment flips, early out if necessary.
		if (Parameter <= PreviousPointParameter)
		{
			return false;
		}
	}

	const int32 NextIndex = Index < NumPoints - 1 ? Index + 1 : INDEX_NONE;
	if (NextIndex != INDEX_NONE)
	{
		const float NextPointParameter = GetParameterAtIndex(NextIndex);

		// Detect possible segment flips, early out if necessary.
		if (Parameter >= NextPointParameter)
		{
			return false;
		}
	}

	ExecuteSetParameter();

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();

	return true;
}

bool FTangentSpline::SetParameters(const TArray<float>& InParameters)
{
	using namespace UE::Geometry::Spline;

	const int32 NumPoints = GetNumberOfPoints();

	if (InParameters.Num() != NumPoints)
	{
		UE_LOGF(LogTangentSpline, Warning, "SetParameters must be provided an array equal in size to the current number of points!")
		return false;
	}

	TArray<FKnot> Knots;

	auto IsEndpoint = [this, NumPoints](const int32 PointIdx) -> bool
	{
		// If we are closed, there are no endpoints.
		if (IsClosedLoop() && NumPoints >= 2)
		{
			return false;
		}

		// If we are open and the first or last point, we are an endpoint.
		return PointIdx == 0 || PointIdx == NumPoints - 1;
	};

	for (int32 ParameterIdx = 0; ParameterIdx < InParameters.Num(); ++ParameterIdx)
	{
		if (ParameterIdx > 0)
		{
			// Detect non-monotonically increasing knot vector, which is invalid.
			if (InParameters[ParameterIdx] <= InParameters[ParameterIdx - 1])
			{
				return false;
			}
		}

		Knots.Emplace(InParameters[ParameterIdx], IsEndpoint(ParameterIdx) ? 4 : 3);
	}

	// True if we need a closing knot
	if (IsClosedLoop() && NumPoints >= 2)
	{
		const float OldClosingSegmentKnotDelta = GetSpline().GetSegmentParameterRange(GetNumberOfSegments() - 1).Length();

		Knots.Last().Multiplicity = 3;
		Knots.Emplace(Knots.Last().Value + OldClosingSegmentKnotDelta, 3);
	}

	GetSpline().SetKnotVector(Knots);

	if (TTangentBezierSpline<FVector>* ScaleSpline = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
	{
		ScaleSpline->SetKnotVector(Knots);
	}

	if (TTangentBezierSpline<FQuat>* RotSpline = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
	{
		RotSpline->SetKnotVector(Knots);
	}

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();

	return true;
}

float FTangentSpline::GetParameterAtIndex(int32 Index) const
{
	return GetSpline().GetParameter(Index);
}

float FTangentSpline::GetParameterAtDistance(float Distance) const
{
	UpdateReparamTable();

	UE::TReadScopeLock Lock(ReparamTableRWLock);

	return ReparamTable.Eval(Distance);
}

float FTangentSpline::GetDistanceAtParameter(float Parameter) const
{
	if (!GetParameterSpace().Contains(Parameter))
	{
		return 0.f;
	}

	UpdateReparamTable();

	UE::TReadScopeLock Lock(ReparamTableRWLock);
	if (ReparamTable.Points.IsEmpty())
	{
		return 0.f;
	}

	if (Parameter <= ReparamTable.Points[0].OutVal)
	{
		return ReparamTable.Points[0].InVal;
	}

	if (Parameter >= ReparamTable.Points.Last().OutVal)
	{
		return ReparamTable.Points.Last().InVal;
	}
	const int32 NumReparamPoints = ReparamTable.Points.Num();
	int32 Low = 0;
	int32 High = NumReparamPoints - 1;
	while (High - Low > 1)
	{
		const int32 Mid = (Low + High) / 2;

		if (ReparamTable.Points[Mid].OutVal < Parameter)
		{
			Low = Mid;
		}
		else
		{
			High = Mid;
		}
	}

	const float Alpha = FInterval1f(ReparamTable.Points[Low].OutVal, ReparamTable.Points[High].OutVal).GetT(Parameter);
	return FMath::Lerp(ReparamTable.Points[Low].InVal, ReparamTable.Points[High].InVal, Alpha);
}

int32 FTangentSpline::GetIndexAtParameter(float Parameter) const
{
	if (!GetParameterSpace().Contains(Parameter))
	{
		return INDEX_NONE;
	}

	int32 MinIndex = 0;
	int32 MaxIndex = GetNumberOfSegments();

	while (MaxIndex - MinIndex > 1)
	{
		int32 MidIndex = (MinIndex + MaxIndex) / 2;

		if (GetSpline().GetParameter(MidIndex) <= Parameter)
		{
			MinIndex = MidIndex;
		}
		else
		{
			MaxIndex = MidIndex;
		}
	}

	return MinIndex;
}

FQuat FTangentSpline::GetOrientation(int32 Index) const
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return FQuat::Identity;
	}

	return GetOrientation(GetParameterAtIndex(Index));
}

FQuat FTangentSpline::GetOrientation(float Param) const
{
	if (!GetParameterSpace().Contains(Param))
	{
		return FQuat::Identity;
	}

	FQuat Rotation = EvaluateRotation(Param);
	Rotation.Normalize();

	FVector Direction = EvaluateDerivative(Param);
	Direction = Direction.GetSafeNormal();

	const FVector UpVector = Rotation.RotateVector(FVector::UpVector);

	return FRotationMatrix::MakeFromXZ(Direction, UpVector).ToQuat();
}

void FTangentSpline::SetOrientation(int32 Index, const FQuat& InOrientation)
{
	if (Index < 0 || Index >= GetNumberOfPoints())
	{
		return;
	}

	// Work backwards to compute the rotation that is currently being applied.
	const FQuat RelativeRotation = InOrientation * GetOrientation(Index).Inverse();

	// Store rotation which transforms the world up vector to the local up vector.
	SetRotation(Index, FQuat::FindBetween(FVector::UpVector, InOrientation.GetUpVector()));

	// Align tangents with rotation, preserving magnitude.
	const FVector OldInTangent = GetInTangent(Index);
	const FVector OldInTangentDirection = OldInTangent.GetSafeNormal();
	const float InTangentMag = OldInTangent.Length();
	const FVector NewInTangentDirection = RelativeRotation.RotateVector(OldInTangentDirection);

	const FVector OldOutTangent = GetOutTangent(Index);
	const FVector OldOutTangentDirection = OldOutTangent.GetSafeNormal();
	const float OutTangentMag = OldOutTangent.Length();
	const FVector NewOutTangentDirection = RelativeRotation.RotateVector(OldOutTangentDirection);

	SetInTangent(Index, InTangentMag * NewInTangentDirection);
	SetOutTangent(Index, OutTangentMag * NewOutTangentDirection);
}

float FTangentSpline::FindNearest(const FVector& InLocation, float& OutSquaredDistance) const
{
	return GetSpline().FindNearest(InLocation, OutSquaredDistance);
}

float FTangentSpline::FindNearestOnSegment(const FVector& InLocation, int32 SegmentIndex, float& OutSquaredDistance) const
{
	return GetSpline().FindNearestOnSegment(InLocation, SegmentIndex, OutSquaredDistance);
}

FVector FTangentSpline::EvaluatePosition(float Parameter) const
{
	if (UE::Spline::GUseLegacyPositionEvaluation)
	{
		RebuildLegacyCurves();
		return PositionCurve.Eval(Parameter);
	}
	else
	{
		return Evaluate(Parameter);
	}
}

FVector FTangentSpline::EvaluateDerivative(float Parameter) const
{
	return GetSpline().GetTangent(Parameter);
}

FQuat FTangentSpline::EvaluateRotation(float Parameter) const
{
	if (UE::Spline::GUseLegacyRotationEvaluation)
	{
		RebuildLegacyCurves();
		return RotationCurve.Eval(Parameter);
	}
	else
	{
		return FTangentSpline::EvaluateAttribute<FQuat>(RotationAttrName, Parameter);
	}
}

FVector FTangentSpline::EvaluateScale(float Parameter) const
{
	if (UE::Spline::GUseLegacyScaleEvaluation)
	{
		RebuildLegacyCurves();
		return ScaleCurve.Eval(Parameter);
	}
	else
	{
		return FTangentSpline::EvaluateAttribute<FVector>(ScaleAttrName, Parameter);
	}
}

void FTangentSpline::UpdateSpline()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTangentSpline::UpdateSpline);

	UpdateTangents();

	// Updates legacy curves based on points, never evaluates internal spline.
	RebuildLegacyCurves();

	++Version;
}

void FTangentSpline::UpdateSpline(const FUpdateSplineParams& Params)
{
	using namespace UE::Geometry::Spline;

	TRACE_CPUPROFILER_EVENT_SCOPE(FTangentSpline::UpdateSpline);

	CachedUpdateSplineParams = Params;

	if (UE::Spline::GSkipDoubleUpdate)
	{
		// Avoid updating the spline twice.
		constexpr bool bUpdateSpline = false;
		SetClosedLoop(Params.bClosedLoop, bUpdateSpline);
	}
	else
	{
		SetClosedLoop(Params.bClosedLoop);
	}

	GetSpline().SetStationaryEndpoints(Params.bStationaryEndpoints);
	ReparamStepsPerSegment = Params.ReparamStepsPerSegment;
	
	if (Params.bClosedLoop)
	{
		const int32 NumPoints = GetNumberOfPoints();

		if (NumPoints > 0)
		{
			const float LastParam = GetParameterAtIndex(NumPoints - 1);
			const float LoopKey = Params.bLoopPositionOverride
				? Params.LoopPosition         // user-specified absolute closing key
				: LastParam + 1.0f;           // legacy default behaviour

			if (LoopKey > LastParam)
			{
				GetSpline().SetParameter(GetNumberOfPoints(), LoopKey);

				if (TTangentBezierSpline<FVector>* ScaleChild = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
				{
					ScaleChild->SetParameter(GetNumberOfPoints(), LoopKey);
				}

				if (TTangentBezierSpline<FQuat>* RotChild = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
				{
					RotChild->SetParameter(GetNumberOfPoints(), LoopKey);
				}
			}
			else
			{
				// Legacy behavior: If the absolute loop position is <= LastParam, the spline is open.

				if (UE::Spline::GSkipDoubleUpdate)
				{
					// Avoid updating the spline twice.
					constexpr bool bUpdateSpline = false;
					SetClosedLoop(false, bUpdateSpline);
				}
				else
				{
					SetClosedLoop(false);
				}
			}
		}
	}

	UpdateSpline();
}

void FTangentSpline::UpdateTangents()
{
	using namespace UE::Geometry::Spline;

	GetSpline().UpdateTangents();

	for (const FName& ChannelName : GetAttributeChannelNamesByValueType<float>())
	{
		if (TTangentBezierSpline<float>* Channel = GetTypedAttributeChannel<TTangentBezierSpline<float>>(ChannelName))
		{
			Channel->UpdateTangents();
		}
	}

	if (TTangentBezierSpline<FVector>* ScaleSpline = GetTypedAttributeChannel<TTangentBezierSpline<FVector>>(ScaleAttrName))
	{
		ScaleSpline->UpdateTangents();
	}

	if (TTangentBezierSpline<FQuat>* RotSpline = GetTypedAttributeChannel<TTangentBezierSpline<FQuat>>(RotationAttrName))
	{
		RotSpline->UpdateTangents();
	}

	MarkReparamTableDirty();
	MarkLegacyCurvesDirty();
}

void FTangentSpline::MarkReparamTableDirty()
{
	ReparamTableNextVersion.Increment();
}

void FTangentSpline::UpdateReparamTable() const
{
	const int32 NumSegments = GetNumberOfSegments();

	auto IsReparamTableDirty = [this]()
		{
			return ReparamTableNextVersion.GetValue() != ReparamTableVersion;
		};

	if (!IsReparamTableDirty() || ReparamStepsPerSegment == 0 || NumSegments == 0)
	{
		return;
	}

	UE::TWriteScopeLock Lock(ReparamTableRWLock);

	if (!IsReparamTableDirty())
	{
		return;
	}

	// We can't rely on the next version to not change during the update.
	const uint32 CachedNextVersion = ReparamTableNextVersion.GetValue();

	ReparamTable.Points.Reset(NumSegments * ReparamStepsPerSegment + 1);
	float AccumulatedLength = 0.0f;
	for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		FInterval1f SegmentParameterRange = GetSegmentParameterRange(SegmentIndex);

		ReparamTable.Points.Emplace(AccumulatedLength, SegmentParameterRange.Min, 0.0f, 0.0f, CIM_Linear);

		for (int32 Step = 1; Step < ReparamStepsPerSegment; ++Step)
		{
			const float SegmentT = static_cast<float>(Step) / ReparamStepsPerSegment;
			ReparamTable.Points.Emplace(GetSegmentLength(SegmentIndex, SegmentT, CachedUpdateSplineParams.Scale3D) + AccumulatedLength, SegmentParameterRange.Interpolate(SegmentT), 0.0f, 0.0f, CIM_Linear);
		}

		AccumulatedLength += GetSegmentLength(SegmentIndex, 1.0f, CachedUpdateSplineParams.Scale3D);
	}

	ReparamTable.Points.Emplace(AccumulatedLength, GetParameterSpace().Max, 0.0f, 0.0f, CIM_Linear);

	ReparamTableVersion = CachedNextVersion;
}

void FTangentSpline::MarkLegacyCurvesDirty()
{
	if (UE::Spline::GImmediatelyUpdateLegacyCurves)
	{
		RebuildLegacyCurves();
	}
	else
	{
		LegacyCurvesNextVersion.Increment();
	}
}

void FTangentSpline::RebuildLegacyCurves() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTangentSpline::RebuildLegacyCurves);

	// Curves are dirty if:
	// 1) Current version is not the same as next version (if next version == version, it has not been dirtied)
	// 2) UE::Spline::GImmediatelyUpdateLegacyCurves == true (legacy curves are updated any time they might be dirty)
	auto AreLegacyCurvesDirty = [this]()
		{
			return LegacyCurvesNextVersion.GetValue() != LegacyCurvesVersion || UE::Spline::GImmediatelyUpdateLegacyCurves;
		};

	if (!AreLegacyCurvesDirty())
	{
		return;
	}

	PositionCurve.Points.Reset(GetNumberOfPoints());
	RotationCurve.Points.Reset(GetNumberOfPoints());
	ScaleCurve.Points.Reset(GetNumberOfPoints());

	using FRotationChannel = UE::Geometry::Spline::TTangentBezierSpline<FQuat>;
	using FScaleChannel = UE::Geometry::Spline::TTangentBezierSpline<FVector>;

	const FRotationChannel* RotationChannel = GetTypedAttributeChannel<FRotationChannel>(RotationAttrName);
	const FScaleChannel* ScaleChannel = GetTypedAttributeChannel<FScaleChannel>(ScaleAttrName);

	for (int32 i = 0; i < GetNumberOfPoints(); ++i)
	{
		const float AttributeParameter = static_cast<float>(i);

		// Additionally populate interp curves here so we don't double loop
		PositionCurve.Points.Emplace(AttributeParameter, GetSpline().GetValue(i), GetSpline().GetTangentIn(i), GetSpline().GetTangentOut(i), ConvertTangentModeToInterpCurveMode(GetSpline().GetTangentModes()[i]));
		RotationCurve.Points.Emplace(AttributeParameter, RotationChannel ? RotationChannel->GetValue(i) : FQuat::Identity);
		RotationCurve.Points.Last().InterpMode = CIM_CurveAuto;
		ScaleCurve.Points.Emplace(AttributeParameter, ScaleChannel ? ScaleChannel->GetValue(i) : FVector::OneVector);
		ScaleCurve.Points.Last().InterpMode = CIM_CurveAuto;
	}

	Algo::SortBy(RotationCurve.Points, &FInterpCurvePoint<FQuat>::InVal);
	Algo::SortBy(ScaleCurve.Points, &FInterpCurvePoint<FVector>::InVal);

	if (IsClosedLoop())
	{
		const float ClosingParameter = GetParameterAtIndex(GetNumberOfPoints());
		PositionCurve.SetLoopKey(ClosingParameter);
		RotationCurve.SetLoopKey(ClosingParameter);
		ScaleCurve.SetLoopKey(ClosingParameter);
	}
	else
	{
		PositionCurve.ClearLoopKey();
		RotationCurve.ClearLoopKey();
		ScaleCurve.ClearLoopKey();
	}

	PositionCurve.AutoSetTangents(0.f, false);
	RotationCurve.AutoSetTangents(0.f, false);
	ScaleCurve.AutoSetTangents(0.f, false);

	LegacyCurvesVersion = LegacyCurvesNextVersion.GetValue();
}

FTangentSpline::FTangentBezierControlPoint FTangentSpline::ConvertToTangentBezierControlPoint(const FSplinePoint& Point) const
{
	FTangentBezierControlPoint NewPoint;
	NewPoint.Position = Point.Position;
	NewPoint.TangentIn = Point.ArriveTangent;
	NewPoint.TangentOut = Point.LeaveTangent;
	NewPoint.TangentMode = ConvertInterpCurveModeToTangentMode(
		ConvertSplinePointTypeToInterpCurveMode(Point.Type));
	return NewPoint;
}

void FTangentSpline::UpdatePointAttributes(const FSplinePoint& Point, int32 PointIndex)
{
	if (PointIndex < 0)
	{
		return;
	}

	SetRotation(PointIndex, Point.Rotation.Quaternion());
	SetScale(PointIndex, Point.Scale);

	MarkLegacyCurvesDirty();
}

float FTangentSpline::ConvertIndexToInternalParameter(int32 Index, float Fraction) const
{
	// Handle empty spline
	if (GetNumberOfPoints() <= 1)
		return 0.0f;

	// Clamp to valid range
	Index = FMath::Clamp(Index, 0, GetNumberOfPoints() - 1);

	// For exact index, just return parameter at that index
	if (FMath::IsNearlyZero(Fraction))
		return GetSpline().GetParameter(Index);

	// For fractional indices, interpolate
	int32 NextIndex = FMath::Min(Index + 1, GetNumberOfPoints() - 1);
	float StartParam = GetSpline().GetParameter(Index);
	float EndParam = GetSpline().GetParameter(NextIndex);

	return FMath::Lerp(StartParam, EndParam, Fraction);
}

int32 FTangentSpline::ConvertInternalParameterToNearestPointIndex(float Parameter) const
{
	// Handle empty spline
	if (GetNumberOfPoints() <= 1)
		return 0;

	// Find segment containing this parameter
	for (int32 i = 0; i < GetNumberOfPoints() - 1; i++)
	{
		float StartParam = GetSpline().GetParameter(i);
		float EndParam = GetSpline().GetParameter(i + 1);

		if (Parameter >= StartParam && Parameter <= EndParam)
		{
			// Return closer index
			float Fraction = (Parameter - StartParam) / FMath::Max(EndParam - StartParam, UE_SMALL_NUMBER);
			return (Fraction <= 0.5f) ? i : (i + 1);
		}
	}

	// Out of range
	return (Parameter < GetSpline().GetParameter(0)) ? 0 : (GetNumberOfPoints() - 1);
}

void FTangentSpline::ValidateRotScale() const
{
	if (!UE::Spline::GValidateRotScale)
	{
		return;
	}

	// Validate child existence
	UE::Geometry::Spline::TTangentBezierSpline<FQuat>* RotSpline = GetTypedAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FQuat>>(RotationAttrName);
	UE::Geometry::Spline::TTangentBezierSpline<FVector>* ScaleSpline = GetTypedAttributeChannel<UE::Geometry::Spline::TTangentBezierSpline<FVector>>(ScaleAttrName);
	ensure(RotSpline && ScaleSpline);

	if (RotSpline && ScaleSpline)
	{
		// Validate number of points
		int32 NumControlPoints = GetNumberOfPoints();
		int32 NumRotPoints = RotSpline->GetNumPoints();
		int32 NumScalePoints = ScaleSpline->GetNumPoints();
		ensure(NumControlPoints == NumRotPoints && NumRotPoints == NumScalePoints);

		// Validate control point -> attr point mapping
		for (int32 i = 0; i < NumControlPoints; i++)
		{
			float InternalParam = GetSpline().GetParameter(i);

			float ExpectedRotParam = MapParameterToChildSpace(RotationAttrName, InternalParam);
			float ActualRotParam = RotSpline->GetParameter(i);
			ensure(FMath::IsNearlyEqual(ExpectedRotParam, ActualRotParam, UE_KINDA_SMALL_NUMBER));

			float ExpectedScaleParam = MapParameterToChildSpace(ScaleAttrName, InternalParam);
			float ActualScaleParam = ScaleSpline->GetParameter(i);
			ensure(FMath::IsNearlyEqual(ExpectedScaleParam, ActualScaleParam, UE_KINDA_SMALL_NUMBER));
		}
	}
}

bool FLegacyTangentSpline::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar << PositionCurve;
	Ar << RotationCurve;
	Ar << ScaleCurve;
	Ar << ReparamTable;

	return true;
}