// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/StringCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StringCurve)


/* FStringCurveKey interface
 *****************************************************************************/

bool FStringCurveKey::operator==(const FStringCurveKey& Curve) const
{
	return ((Time == Curve.Time) && (Value == Curve.Value));
}


bool FStringCurveKey::operator!=(const FStringCurveKey& Other) const
{
	return !(*this == Other);
}


bool FStringCurveKey::Serialize(FArchive& Ar)
{
	Ar << Time << Value;
	return true;
}


/* FStringCurve interface
 *****************************************************************************/

FKeyHandle FStringCurve::AddKey(float InTime, const FString& InValue, FKeyHandle KeyHandle)
{
	int32 Index = 0;

	// insert key
	for(; Index < Keys.Num() && Keys[Index].Time < InTime; ++Index);
	Keys.Insert(FStringCurveKey(InTime, InValue), Index);

	KeyHandlesToIndices.Add(KeyHandle, Index);

	return GetKeyHandle(Index);
}


void FStringCurve::DeleteKey(FKeyHandle KeyHandle)
{
	// remove key
	int32 Index = GetIndex(KeyHandle);
	Keys.RemoveAt(Index);

	KeyHandlesToIndices.Remove(KeyHandle);
}


FString FStringCurve::Eval(float Time, const FString& InDefaultValue) const
{
	// If the default value is empty, use the incoming default value
	FString ReturnVal = DefaultValue.IsEmpty() ? InDefaultValue : DefaultValue;

	if (Keys.Num() == 0 || Time < Keys[0].Time)
	{
		// If no keys in curve, or the time is before the first key, return the Default value.
	}
	else if (Keys.Num() < 2 || Time < Keys[0].Time)
	{
		// There is only one key or the time is before the first value. Return the first value
		ReturnVal = Keys[0].Value;
	}
	else if (Time < Keys[Keys.Num() - 1].Time)
	{
		// The key is in the range of Key[0] to Keys[Keys.Num()-1].  Find it by searching
		for (int32 i = 0; i < Keys.Num(); ++i)
		{
			if (Time < Keys[i].Time)
			{
				ReturnVal = Keys[FMath::Max(0, i - 1)].Value;
				break;
			}
		}
	}
	else
	{
		// Key is beyond the last point in the curve.  Return it's value
		ReturnVal = Keys[Keys.Num() - 1].Value;
	}

	return ReturnVal;
}


FKeyHandle FStringCurve::FindKey(float KeyTime, float KeyTimeTolerance) const
{
	int32 Start = 0;
	int32 End = Keys.Num()-1;

	// Binary search since the keys are in sorted order
	while (Start <= End)
	{
		int32 TestPos = Start + (End-Start) / 2;
		float TestKeyTime = Keys[TestPos].Time;

		if (FMath::IsNearlyEqual(TestKeyTime, KeyTime, KeyTimeTolerance))
		{
			return GetKeyHandle(TestPos);
		}

		if (TestKeyTime < KeyTime)
		{
			Start = TestPos+1;
		}
		else
		{
			End = TestPos-1;
		}
	}

	return FKeyHandle::Invalid();
}


FStringCurveKey& FStringCurve::GetKey(FKeyHandle KeyHandle)
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}


FStringCurveKey FStringCurve::GetKey(FKeyHandle KeyHandle) const
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}


float FStringCurve::GetKeyTime(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Time;
}


FString FStringCurve::GetKeyValue(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return FString();
	}

	return GetKey(KeyHandle).Value;
}


void FStringCurve::SetKeyTime(FKeyHandle KeyHandle, float NewTime)
{
	if (IsKeyHandleValid(KeyHandle))
	{
		const FStringCurveKey OldKey = GetKey(KeyHandle);

		DeleteKey(KeyHandle);
		AddKey(NewTime, OldKey.Value, KeyHandle);

		// Copy all properties from old key, but then fix time to be the new time
		FStringCurveKey& NewKey = GetKey(KeyHandle);
		NewKey = OldKey;
		NewKey.Time = NewTime;
	}
}

void FStringCurve::SetKeyValue(FKeyHandle KeyHandle, FString NewValue)
{
	if (IsKeyHandleValid(KeyHandle))
	{
		GetKey(KeyHandle).Value = NewValue;
	}
}

FKeyHandle FStringCurve::UpdateOrAddKey(float InTime, const FString& InValue, float KeyTimeTolerance)
{
	// Search for a key that already exists at the time and if found, update its value
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		float KeyTime = Keys[KeyIndex].Time;

		if (FMath::IsNearlyEqual(KeyTime, InTime, KeyTimeTolerance))
		{
			Keys[KeyIndex].Value = InValue;

			return GetKeyHandle(KeyIndex);
		}

		if (KeyTime > InTime)
		{
			// All the rest of the keys exist after the key we want to add
			// so there is no point in searching
			break;
		}
	}

	// a key wasn't found, add it now
	return AddKey(InTime, InValue);
}

void FStringCurve::RemoveRedundantKeys()
{
	TArray<FKeyHandle> KeyHandlesToRemove;
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		if (KeyIndex + 2 < Keys.Num())
		{
			const FStringCurveKey& CurrentKey = Keys[KeyIndex];
			const FStringCurveKey& NextKeyOne = Keys[KeyIndex + 1];
			const FStringCurveKey& NextKeyTwo = Keys[KeyIndex + 2];

			if (CurrentKey.Value == NextKeyOne.Value && NextKeyOne.Value == NextKeyTwo.Value)
			{
				KeyHandlesToRemove.Add(GetKeyHandle(KeyIndex + 1));
			}
		}
	}

	for (const FKeyHandle& Handle : KeyHandlesToRemove)
	{
		DeleteKey(Handle);
	}

	// Const value, remove keys and set default value
	if (Keys.Num() == 1)
	{
		DefaultValue = Keys[0].Value;
		DeleteKey(GetKeyHandle(0));
	}
}

