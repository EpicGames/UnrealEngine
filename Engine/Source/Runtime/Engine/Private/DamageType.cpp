// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/DamageType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DamageType)

UDamageType::UDamageType(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DamageImpulse = 800.0f;
	DestructibleImpulse = 800.0f;
	DestructibleDamageSpreadScale = 1.0f;
	bScaleMomentumByMass = true;
	DamageFalloff = 1.0f;
}

