// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DataAsset.h"

#include "Misc/PackageName.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "Engine/AssetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataAsset)

const FName UDataAsset::NativeClassTag(TEXT("NativeClass"));

void UDataAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	// Walk up until we find a native class (ie 'while they are BP classes')
	UClass* NativeParentClass = GetClass();
	while (!NativeParentClass->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
	{
		NativeParentClass = NativeParentClass->GetSuperClass();
	}
	check(NativeParentClass);

	Context.AddTag(FAssetRegistryTag(NativeClassTag, FObjectPropertyBase::GetExportPath(NativeParentClass), FAssetRegistryTag::TT_Alphabetical));

	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR
FText UDataAsset::GetDisplayNameText() const
{
	return FText::AsCultureInvariant(GetName());
}
#endif

#if WITH_EDITORONLY_DATA
void UDataAsset::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();
	Super::Serialize(Record);

	if (UnderlyingArchive.IsLoading() && (UnderlyingArchive.UEVer() < VER_UE4_ADD_TRANSACTIONAL_TO_DATA_ASSETS))
	{
		SetFlags(RF_Transactional);
	}
}

void UPrimaryDataAsset::UpdateAssetBundleData()
{
	// By default parse the metadata
	if (UAssetManager::IsInitialized())
	{
		AssetBundleData.Reset();
		UAssetManager::Get().InitializeAssetBundlesFromMetadata(this, AssetBundleData);
	}
}

void UPrimaryDataAsset::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	UpdateAssetBundleData();

	if (UAssetManager::IsInitialized())
	{
		// Bundles may have changed, refresh
		UAssetManager::Get().RefreshAssetData(this);
	}
}
#endif

FPrimaryAssetId UPrimaryDataAsset::GetPrimaryAssetId() const
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UClass* BestPrimaryAssetTypeClass = nullptr;
		UClass* SearchPrimaryAssetTypeClass = GetClass()->GetSuperClass();

		// If this is a native class or immediate child of PrimaryDataAsset, return invalid as we are a type ourselves
		if (GetClass()->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic) || SearchPrimaryAssetTypeClass == UPrimaryDataAsset::StaticClass())
		{
			return FPrimaryAssetId();
		}

		// Starting with parent, look up the hierarchy for a class that is either native, or a blueprint class immediately below PrimaryDataAsset
		while (SearchPrimaryAssetTypeClass)
		{
			if (SearchPrimaryAssetTypeClass->GetSuperClass() == UPrimaryDataAsset::StaticClass())
			{
				// If our parent is this base class, return this as the best class
				BestPrimaryAssetTypeClass = SearchPrimaryAssetTypeClass;
				break;
			}
			else if (SearchPrimaryAssetTypeClass->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
			{
				// Our parent is the first native class found, use that
				BestPrimaryAssetTypeClass = SearchPrimaryAssetTypeClass;
				break;
			}
			else
			{
				SearchPrimaryAssetTypeClass = SearchPrimaryAssetTypeClass->GetSuperClass();
			}
		}

		if (BestPrimaryAssetTypeClass)
		{
			// If this is a native class use the raw name if it's a blueprint use the package name as it will be missing _C
			FName PrimaryAssetType = BestPrimaryAssetTypeClass->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic) ? BestPrimaryAssetTypeClass->GetFName() : FPackageName::GetShortFName(BestPrimaryAssetTypeClass->GetOutermost()->GetFName());

			return FPrimaryAssetId(PrimaryAssetType, FPackageName::GetShortFName(GetOutermost()->GetName()));
		}

		// No valid parent class found, return invalid
		return FPrimaryAssetId();
	}


	// Data assets use Class and ShortName by default, there's no inheritance so class works fine
	return FPrimaryAssetId(GetClass()->GetFName(), GetFName());
}

void UPrimaryDataAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	FAssetBundleData OldData = AssetBundleData;
	
	UpdateAssetBundleData();

	if (UAssetManager::IsInitialized() && OldData != AssetBundleData)
	{
		// Bundles changed, refresh
		UAssetManager::Get().RefreshAssetData(this);
	}
#endif
}

