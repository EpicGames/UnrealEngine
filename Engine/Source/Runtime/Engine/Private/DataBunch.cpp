// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DataBunch.cpp: Unreal bunch (sub-packet) functions.
=============================================================================*/

#include "Net/DataBunch.h"
#include "Engine/ControlChannel.h"
#include "Engine/NetConnection.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"

const int32 MAX_BUNCH_SIZE = 1024 * 1024;

/*-----------------------------------------------------------------------------
	FInBunch implementation.
-----------------------------------------------------------------------------*/

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FInBunch::FInBunch( UNetConnection* InConnection, uint8* Src, int64 CountBits )
:	FNetBitReader	(InConnection->PackageMap, Src, CountBits)
,	PacketId	( 0 )
,	Next ( nullptr )
,	Connection ( InConnection )
,	ChIndex ( 0 )
,	ChName ( NAME_None )
,	ChSequence ( 0 )
,	bOpen ( 0 )
,	bClose ( 0 )
,	bIsReplicationPaused( 0 )
,	bReliable ( 0 )
,	bPartial ( 0 )
,	bPartialInitial ( 0 )
,	bPartialFinal ( 0 )
,	bHasPackageMapExports ( 0 )
,	bHasMustBeMappedGUIDs ( 0 )
,	bPartialCustomExportsFinal( 0 )
,	bIgnoreRPCs ( 0 )
,	CloseReason( EChannelCloseReason::Destroyed )
{
	check(Connection);
	// Match the byte swapping settings of the connection
	SetByteSwapping(Connection->bNeedsByteSwapping);
}

/** Copy constructor but with optional parameter to not copy buffer */
FInBunch::FInBunch( FInBunch &InBunch, bool CopyBuffer )
{
	PacketId =	InBunch.PacketId;
	Next =	InBunch.Next;
	Connection = InBunch.Connection;
	ChIndex = InBunch.ChIndex;
	ChName = InBunch.ChName;
	ChSequence = InBunch.ChSequence;
	bOpen =	InBunch.bOpen;
	bClose = InBunch.bClose;
	bIsReplicationPaused = InBunch.bIsReplicationPaused;
	bReliable =	InBunch.bReliable;
	bPartial = InBunch.bPartial;
	bPartialInitial = InBunch.bPartialInitial;
	bPartialFinal =	InBunch.bPartialFinal;
	bHasPackageMapExports = InBunch.bHasPackageMapExports;
	bHasMustBeMappedGUIDs =	InBunch.bHasMustBeMappedGUIDs;
	bPartialCustomExportsFinal = InBunch.bPartialCustomExportsFinal;
	bIgnoreRPCs = InBunch.bIgnoreRPCs;
	CloseReason = InBunch.CloseReason;
	PackageMap = InBunch.PackageMap;

	if (CopyBuffer)
	{
		FBitReader::operator=(InBunch);
	}

	Pos = 0;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FInBunch::CountMemory(FArchive& Ar) const
{
	for (const FInBunch* Current = this; Current; Current = Current->Next)
	{
		Current->FNetBitReader::CountMemory(Ar);
		const SIZE_T MemberSize = sizeof(*this) - sizeof(FNetBitReader);
		Ar.CountBytes(MemberSize, MemberSize);
	}
}

uint32 FInBunch::EngineNetVer() const
{
	return Connection->GetNetworkCustomVersion(FEngineNetworkCustomVersion::Guid);
}

uint32 FInBunch::GameNetVer() const
{
	return Connection->GetNetworkCustomVersion(FGameNetworkCustomVersion::Guid);
}

void FInBunch::SetEngineNetVer(const uint32 InEngineNetVer)
{
	//ensureMsgf(false, TEXT("SetEngineNetVer should not be called on FInBunch"));
}

void FInBunch::SetGameNetVer(const uint32 InGameNetVer)
{
	//ensureMsgf(false, TEXT("SetGameNetVer should not be called on FInBunch"));
}

/*-----------------------------------------------------------------------------
	FOutBunch implementation.
-----------------------------------------------------------------------------*/

//
// Construct an outgoing bunch for a channel.
// It is ok to either send or discard an FOutbunch after construction.
//
FOutBunch::FOutBunch()
: FNetBitWriter( 0 )
{}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FOutBunch::FOutBunch( UChannel* InChannel, bool bInClose )
:	FNetBitWriter	( InChannel->Connection->PackageMap, InChannel->Connection->GetMaxSingleBunchSizeBits())
,	Next		( nullptr )
,	Channel		( InChannel )
,	Time		( 0 )
,	ChIndex		( InChannel->ChIndex )
,	ChName		( InChannel->ChName )
,	ChSequence	( 0 )
,	PacketId	( 0 )
,	ReceivedAck	( 0 )
,	bOpen		( 0 )
,	bClose		( bInClose )
,	bIsReplicationPaused	( 0 )
,	bReliable	( 0 )
,	bPartial	( 0 )
,	bPartialInitial			( 0 )
,	bPartialFinal			( 0 )
,	bHasPackageMapExports	( 0 )
,	bHasMustBeMappedGUIDs	( 0 )
,	bPartialCustomExportsFinal	( 0 )
,	bOutWantsFullInitState	( 0 )
,	CloseReason( EChannelCloseReason::Destroyed )
{
	checkSlow(!Channel->Closing);
	checkSlow(Channel->Connection->Channels[Channel->ChIndex]==Channel);

	// Match the byte swapping settings of the connection
	SetByteSwapping(Channel->Connection->bNeedsByteSwapping);

	// Reserve channel and set bunch info.
	if( Channel->NumOutRec >= RELIABLE_BUFFER-1+bClose )
	{
		SetOverflowed(-1);
		return;
	}
}
FOutBunch::FOutBunch( UPackageMap *InPackageMap, int64 MaxBits )
:	FNetBitWriter	( InPackageMap, MaxBits )
,	Next		( nullptr )
,	Channel		( nullptr )
,	Time		( 0 )
,	ChIndex		( 0 )
,	ChName		( )
,	ChSequence	( 0 )
,	PacketId	( 0 )
,	ReceivedAck	( 0 )
,	bOpen		( 0 )
,	bClose		( 0 )
,   bIsReplicationPaused	( 0 )
,	bReliable	( 0 )
,	bPartial	( 0 )
,	bPartialInitial ( 0 )
,	bPartialFinal	( 0 )
,	bHasPackageMapExports	( 0 )
,	bHasMustBeMappedGUIDs	( 0 )
,	bPartialCustomExportsFinal	( 0 )
,	bOutWantsFullInitState	( 0 )
,	CloseReason( EChannelCloseReason::Destroyed )
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FOutBunch::FOutBunch(int64 InMaxBits)
: FOutBunch(static_cast<UPackageMap*>(nullptr), InMaxBits)
{
}

void FOutBunch::Reset()
{
	FNetBitWriter::Reset();
	NetTokensPendingExport.Reset();
}

void FOutBunch::CountMemory(FArchive& Ar) const
{
	for (const FOutBunch* Current = this; Current; Current = Current->Next)
	{
		Current->FNetBitWriter::CountMemory(Ar);
		const SIZE_T MemberSize = sizeof(*this) - sizeof(FNetBitWriter);
		Ar.CountBytes(MemberSize, MemberSize);
	}
}

FOutBunch::~FOutBunch()
{
	UE_NET_TRACE_DESTROY_COLLECTOR(TraceCollector.Get());
}

FControlChannelOutBunch::FControlChannelOutBunch(UChannel* InChannel, bool bClose)
	: FOutBunch(InChannel, bClose)
{
	checkSlow(Cast<UControlChannel>(InChannel) != nullptr);
	// control channel bunches contain critical handshaking/synchronization and should always be reliable
	bReliable = true;
}

