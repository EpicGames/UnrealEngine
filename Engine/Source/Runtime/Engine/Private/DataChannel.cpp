// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DataChannel.cpp: Unreal datachannel implementation.
=============================================================================*/

#include "Net/DataChannel.h"
#include "Containers/StaticBitArray.h"
#include "Engine/GameInstance.h"
#include "Engine/ChildConnection.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Interfaces/ReplicatedObjectInterface.h"
#include "Misc/MemStack.h"
#include "Misc/ScopeExit.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "Net/Core/Misc/GuidReferences.h"
#include "Net/Core/NetCoreModule.h"
#include "Net/Core/Connection/NetEnums.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "EngineStats.h"
#include "Engine/Engine.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "DrawDebugHelpers.h"
#include "Net/NetworkProfiler.h"
#include "Net/DataReplication.h"
#include "Net/NetworkMetricsDatabase.h"
#include "Net/NetworkMetricsDefs.h"
#include "Engine/ActorChannel.h"
#include "Engine/ControlChannel.h"
#include "Engine/DemoNetDriver.h"
#include "Engine/NetworkObjectList.h"
#include "Stats/StatsMisc.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "Net/Subsystems/NetworkSubsystem.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/NetSubObjectRegistryGetter.h"
#include "HAL/LowLevelMemStats.h"
#include "Net/NetPing.h"
#include "PacketHandler.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"

#if UE_NET_REPACTOR_NAME_DEBUG
#include "Net/NetNameDebug.h"
#endif

DEFINE_LOG_CATEGORY(LogNet);
DEFINE_LOG_CATEGORY(LogNetSubObject);
DEFINE_LOG_CATEGORY(LogRep);
DEFINE_LOG_CATEGORY(LogNetPlayerMovement);
DEFINE_LOG_CATEGORY(LogNetTraffic);
DEFINE_LOG_CATEGORY(LogNetLifecycle);
DEFINE_LOG_CATEGORY(LogRepTraffic);
DEFINE_LOG_CATEGORY(LogNetDormancy);
DEFINE_LOG_CATEGORY(LogSecurity);
DEFINE_LOG_CATEGORY_STATIC(LogNetPartialBunch, Warning, All);

DECLARE_CYCLE_STAT(TEXT("ActorChan_CleanUp"), Stat_ActorChanCleanUp, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("Channel ReceivedRawBunch"), Stat_ChannelReceivedRawBunch, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("ActorChan_FindOrCreateRep"), Stat_ActorChanFindOrCreateRep, STATGROUP_Net);

DECLARE_LLM_MEMORY_STAT(TEXT("NetChannel"), STAT_NetChannelLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(NetChannel, NAME_None, TEXT("Networking"), GET_STATFNAME(STAT_NetChannelLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));

extern int32 GDoReplicationContextString;
extern int32 GNetDormancyValidate;
extern bool GbNetReuseReplicatorsForDormantObjects;

TAutoConsoleVariable<int32> CVarNetReliableDebug(
	TEXT("net.Reliable.Debug"),
	0,
	TEXT("Print all reliable bunches sent over the network\n")
	TEXT(" 0: no print.\n")
	TEXT(" 1: Print bunches as they are sent.\n")
	TEXT(" 2: Print reliable bunch buffer each net update"),
	ECVF_Default);

static TAutoConsoleVariable<int> CVarNetProcessQueuedBunchesMillisecondLimit(
	TEXT("net.ProcessQueuedBunchesMillisecondLimit"),
	30,
	TEXT("Time threshold for processing queued bunches. If it takes longer than this in a single frame, wait until the next frame to continue processing queued bunches. For unlimited time, set to 0."));

static TAutoConsoleVariable<int32> CVarNetInstantReplayProcessQueuedBunchesMillisecondLimit(
	TEXT("net.InstantReplayProcessQueuedBunchesMillisecondLimit"),
	8,
	TEXT("Time threshold for processing queued bunches during instant replays. If it takes longer than this in a single frame, wait until the next frame to continue processing queued bunches. For unlimited time, set to 0."));

int32 GCVarNetPartialBunchReliableThreshold = 8;
FAutoConsoleVariableRef CVarNetPartialBunchReliableThreshold(
	TEXT("net.PartialBunchReliableThreshold"),
	GCVarNetPartialBunchReliableThreshold,
	TEXT("If a bunch is broken up into this many partial bunches are more, we will send it reliable even if the original bunch was not reliable. Partial bunches are atonmic and must all make it over to be used"));

int32 GSkipReplicatorForDestructionInfos = 1;
FAutoConsoleVariableRef CVarNetSkipReplicatorForDestructionInfos(
	TEXT("net.SkipReplicatorForDestructionInfos"),
	GSkipReplicatorForDestructionInfos,
	TEXT("If enabled, skip creation of object replicator in SetChannelActor when we know there is no content payload and we're going to immediately destroy the actor."));

// Fairly large number, and probably a bad idea to even have a bunch this size, but want to be safe for now and not throw out legitimate data
static int32 NetMaxConstructedPartialBunchSizeBytes = 1024 * 64;
static FAutoConsoleVariableRef CVarNetMaxConstructedPartialBunchSizeBytes(
	TEXT("net.MaxConstructedPartialBunchSizeBytes"),
	NetMaxConstructedPartialBunchSizeBytes,
	TEXT("The maximum size allowed for Partial Bunches.")
);

static float DormancyHysteresis = 0;
static FAutoConsoleVariableRef CVarDormancyHysteresis(
	TEXT("net.DormancyHysteresis"),
	DormancyHysteresis,
	TEXT("When > 0, represents the time we'll wait before letting a channel become fully dormant (in seconds). This can prevent churn when objects are going in and out of dormant more frequently than normal.")
);

namespace UE::Net
{
	extern int32 FilterGuidRemapping;
	extern bool bDiscardTornOffActorRPCs;

	static float QueuedBunchTimeoutSeconds = 30.0f;
	static FAutoConsoleVariableRef CVarQueuedBunchTimeoutSeconds(
		TEXT("net.QueuedBunchTimeoutSeconds"),
		QueuedBunchTimeoutSeconds,
		TEXT("Time in seconds to wait for queued bunches on a channel to flush before logging a warning."));

	bool bNetReplicateOnlyBeginPlay = false;
	static FAutoConsoleVariableRef CVarNetReplicateOnlyBeginPlay(
		TEXT("net.ReplicateOnlyBeginPlay"),
		bNetReplicateOnlyBeginPlay,
		TEXT("Only allow property replication of actors that had BeginPlay called on them."), ECVF_Default);

#if SUBOBJECT_TRANSITION_VALIDATION
	static bool GCVarCompareSubObjectsReplicated = false;
	static FAutoConsoleVariableRef CVarCompareSubObjectsReplicated(
		TEXT("net.SubObjects.CompareWithLegacy"),
		GCVarCompareSubObjectsReplicated,
		TEXT("When turned on we will collect the subobjects replicated by the ReplicateSubObjects method and compare them with the ones replicated via the Actor's registered list. If a divergence is detected it will trigger an ensure."));

	static TAutoConsoleVariable<int32> CVarLogAllSubObjectComparisonErrors(
		TEXT("net.SubObjects.LogAllComparisonErrors"),
		0,
		TEXT("If enabled log all the errors detected by the CompareWithLegacy cheat. Otherwise only the first ensure triggered gets logged."));

	static bool GCVarDetectDeprecatedReplicateSubObjects = false;
	static FAutoConsoleVariableRef CVarDetectDeprecatedReplicateSubObjects(
		TEXT("net.SubObjects.DetectDeprecatedReplicatedSubObjects"),
		GCVarDetectDeprecatedReplicateSubObjects,
		TEXT("When turned on, we trigger an ensure if we detect a ReplicateSubObjects() function is still implemented in a class that is using the new SubObject list."));
#endif

	static bool bEnableNetInitialSubObjects = true;
	static FAutoConsoleVariableRef CVarEnableNetInitialSubObjects(
		TEXT("net.EnableNetInitialSubObjects"),
		bEnableNetInitialSubObjects,
		TEXT("Enables new SubObjects to set bNetInitial to true to make sure all replicated properties are replicated.")
	);

	static bool bPushModelValidateSkipUpdate = false;
	static FAutoConsoleVariableRef CVarNetPushModelValidateSkipUpdate(
		TEXT("net.PushModelValidateSkipUpdate"),
		bPushModelValidateSkipUpdate,
		TEXT("If enabled, detect when we thought we could skip an object replication based on push model state, but we sent data anyway."), ECVF_Default);

#if UE_NET_REPACTOR_NAME_DEBUG
	/** Whether or not the currently executing ReplicateActor code has met the randomized chance for enabling name debugging */
	static bool GMetAsyncDemoNameDebugChance = false;
#endif

	static bool bSkipDestroyNetStartupActorsOnChannelCloseDueToLevelUnloaded = true;
	static FAutoConsoleVariableRef CVarSkipDestroyNetStartupActorsOnChannelCloseDueToLevelUnloaded(
		TEXT("net.SkipDestroyNetStartupActorsOnChannelCloseDueToLevelUnloaded"),
		bSkipDestroyNetStartupActorsOnChannelCloseDueToLevelUnloaded,
		TEXT("Controls if Actor that is a NetStartUpActor assosciated with the channel is destroyed or not when we receive a channel close with ECloseReason::LevelUnloaded."), ECVF_Default);

	static float QueuedBunchTimeFailsafeSeconds = 2.0f;
	static FAutoConsoleVariableRef CVarNetQueuedBunchTimeFailsafeSeconds(
		TEXT("net.QueuedBunchTimeFailsafeSeconds"),
		QueuedBunchTimeFailsafeSeconds,
		TEXT("Amount of time in seconds to wait with queued bunches before forcibly processing them all, ignoring the NetDriver's HasExceededIncomingBunchFrameProcessingTime."));

	// bTearOff is private but we still might need to adjust the flag on clients based on the channel close reason
	class FTearOffSetter final
	{
		FTearOffSetter() = delete;

	public:
		static void SetTearOff(AActor* Actor)
		{
			Actor->bTearOff = true;
		}
	};
}; // namespace UE::Net

template<typename T>
static const bool IsBunchTooLarge(UNetConnection* Connection, T* Bunch)
{
	return !Connection->IsUnlimitedBunchSizeAllowed() && Bunch != nullptr && Bunch->GetNumBytes() > NetMaxConstructedPartialBunchSizeBytes;
}

namespace DataChannelInternal
{
	/**
	* Track the owner of the next subobjects replicated via ReplicateSubObject.
	* Nulled out after every call to UActorChannel::ReplicateActor()
	*/
	UObject* CurrentSubObjectOwner = nullptr;

	/**
	* When enabled we will ignore requests to replicate a subobject via the public ReplicateSubObject function.
	* This occurs when the subobjects belong to an ActorComponent converted to the registered list, but the owner does not support it yet so we still call ReplicateSubObjects on the actor.
	* This flag is controlled via UActorChannel::SetCurrentSubObjectOwner, so it is important for users to set the owner properly if they are controlling component replication themselves.
	*/
	bool bIgnoreLegacyReplicateSubObject = false;

	/** When enabled store all subobjects written into the bunch in the ReplicatedSubObjectsTracker array */
	bool bTrackReplicatedSubObjects = false;

	/**
	* This flag is true when an actor using the new subobject list is calling the legacy function to look for divergences between the old and new method.
	* This ensures that the subobjects replicated the old way are only collected and not replicated.
	*/
	bool bTestingLegacyMethodForComparison = false;

#if SUBOBJECT_TRANSITION_VALIDATION
    struct FSubObjectReplicatedInfo
	{
		// The replicated subobject
		UObject* SubObject = nullptr;
		// The owner of the subobject, usually an actor or actor component
		UObject* SubObjectOwner = nullptr;

		FSubObjectReplicatedInfo(UObject* InSubObject) : SubObject(InSubObject), SubObjectOwner(CurrentSubObjectOwner) {}

		bool operator==(const FSubObjectReplicatedInfo& rhs) const { return SubObject == rhs.SubObject; }

		FString Describe(AActor* ReplicatedActor) const
		{
			return FString::Printf(TEXT("Subobject %s(%s)::%s [owner %s]"),
				*ReplicatedActor->GetName(),
                ReplicatedActor->HasActorBegunPlay() ? TEXT("BegunPlay") : ReplicatedActor->IsActorBeginningPlay() ? TEXT("BeginningPlay") : TEXT("HasNotBegunPlay"),
				*SubObject->GetName(),
				*GetPathNameSafe(SubObjectOwner));
		}
	};

	/**
	* List of replicated subobjects collected via Actor::ReplicateSubObjects.
	* This list is static to prevent memory overhead on every actor channel.
	* It is emptied at the end of every call to UActorChannel::ReplicateActor()
	*/
	TArray<FSubObjectReplicatedInfo> LegacySubObjectsCollected;

	/**
	* The subobjects actually replicated into the bunch.
	* This list is static to prevent memory overhead on every actor channel.
	* It is emptied at the end of every call to UActorChannel::ReplicateActor()
	*/
	TArray<FSubObjectReplicatedInfo> ReplicatedSubObjectsTracker;
#endif

	/**
	* Temporary pointer to the NetworkSubsystem that gets set and released in ReplicateActor
	* Prevents having to grab it for every unique subobject condition test.
	*/
	static UNetworkSubsystem* CachedNetworkSubsystem = nullptr;
}

/*-----------------------------------------------------------------------------
	UChannel implementation.
-----------------------------------------------------------------------------*/

void UChannel::Init(UNetConnection* InConnection, int32 InChIndex, EChannelCreateFlags CreateFlags)
{
	// if child connection then use its parent
	if (InConnection->GetUChildConnection() != NULL)
	{
		Connection = ((UChildConnection*)InConnection)->Parent;
	}
	else
	{
		Connection = InConnection;
	}
	ChIndex			= InChIndex;
	OpenedLocally	= EnumHasAnyFlags(CreateFlags, EChannelCreateFlags::OpenedLocally);
	OpenPacketId	= FPacketIdRange();
	bPausedUntilReliableACK = 0;
	SentClosingBunch = 0;
}


void UChannel::SetClosingFlag()
{
	Closing = 1;
}


int64 UChannel::Close(EChannelCloseReason Reason)
{
	check(OpenedLocally || ChIndex == 0);		// We are only allowed to close channels that we opened locally (except channel 0, so the server can notify disconnected clients)
	check(Connection->Channels[ChIndex]==this);

	int64 NumBits = 0;

	if ( !Closing && ( Connection->GetConnectionState() == USOCK_Open || Connection->GetConnectionState() == USOCK_Pending || (Connection->GetConnectionState() == USOCK_Closing && ChIndex == 0)) && !SentClosingBunch)
	{
		if ( ChIndex == 0 )
		{
			UE_LOGF(LogNet, Log, "UChannel::Close: Sending CloseBunch. ChIndex == 0. Name: %ls", *Describe());
		}

		UE_LOGF(LogNetDormancy, Verbose, "UChannel::Close: Sending CloseBunch. Reason: %ls, %ls", LexToString(Reason), *Describe());

		// Send a close notify, and wait for ack.
		PacketHandler* Handler = Connection->Handler.Get();

		if ((Handler == nullptr || Handler->IsFullyInitialized()) && Connection->HasReceivedClientPacket())
		{
			FOutBunch CloseBunch( this, 1 );

			SentClosingBunch = 1; //in case this send ends up failing and trying to reach back to close the connection, don't allow recursion.

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			CloseBunch.DebugString = FString::Printf(TEXT("%.2f Close: %s"), Connection->Driver->GetElapsedTime(), *Describe());
#endif
			check(!CloseBunch.IsError());
			check(CloseBunch.bClose);
			CloseBunch.bReliable = 1;
			CloseBunch.CloseReason = Reason;
			SendBunch( &CloseBunch, 0 );
			NumBits = CloseBunch.GetNumBits();
		}
	}

	return NumBits;
}

void UChannel::ConditionalCleanUp( const bool bForDestroy, EChannelCloseReason CloseReason )
{
	if (IsValidChecked(this) && !bPooled)
	{
		// CleanUp can return false to signify that we shouldn't mark pending kill quite yet
		// We'll need to call cleanup again later on
		UNetDriver* Driver = Connection ? Connection->GetDriver() : nullptr;
		if ( CleanUp( bForDestroy, CloseReason ) )
		{
			// Tell the driver that this channel is now cleaned up and can be returned to a pool, if appropriate
			if (Driver && !bForDestroy)
			{
				Driver->ReleaseToChannelPool(this);
			}

			// If we were not added to a pool, mark pending kill and allow the channel to GC
			if (!bPooled)
			{
				MarkAsGarbage();
			}
		}
	}
}

bool UChannel::CleanUp(const bool bForDestroy, EChannelCloseReason CloseReason)
{

	checkSlow(Connection != NULL);
	checkSlow(Connection->Channels[ChIndex] == this);

	// if this is the control channel, make sure we properly killed the connection
	if (ChIndex == 0 && !Closing)
	{
		UE_LOGF(LogNet, Log, "UChannel::CleanUp: ChIndex == 0. Closing connection. %ls", *Describe());
		Connection->Close(ENetCloseResult::ControlChannelClose);
	}

	// remember sequence number of first non-acked outgoing reliable bunch for this slot
	if (OutRec != NULL && !Connection->IsInternalAck())
	{
		Connection->PendingOutRec[ChIndex] = OutRec->ChSequence;
		//UE_LOGF(LogNetTraffic, Log, "%i save pending out bunch %i",ChIndex,Connection->PendingOutRec[ChIndex]);
	}
	// Free any pending incoming and outgoing bunches.
	for (FOutBunch* Out = OutRec, *NextOut; Out != NULL; Out = NextOut)
	{
		NextOut = Out->Next;
		delete Out;
	}
	OutRec = nullptr;
	for (FInBunch* In = InRec, *NextIn; In != NULL; In = NextIn)
	{
		NextIn = In->Next;
		delete In;
	}
	InRec = nullptr;
	if (InPartialBunch != NULL)
	{
		delete InPartialBunch;
		InPartialBunch = NULL;
	}

	if (ChIndex != INDEX_NONE && Connection->IsReservingDestroyedChannels())
	{
		Connection->AddReservedChannel(ChIndex);
	}

	// Remove from connection's channel table.
	verifySlow(Connection->OpenChannels.Remove(this) == 1);
	Connection->StopTickingChannel(this);
	Connection->Channels[ChIndex] = NULL;
	Connection = NULL;

	return true;
}


void UChannel::BeginDestroy()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ConditionalCleanUp(true, EChannelCloseReason::Destroyed);
	}

	Super::BeginDestroy();
}

void UChannel::Serialize(FArchive& Ar)
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UChannel::Serialize");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Super", Super::Serialize(Ar));

	if (Ar.IsCountingMemory())
	{
		if (InRec)
		{
			GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("InRec", InRec->CountMemory(Ar));
		}

		if (OutRec)
		{
			GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("OutRec", OutRec->CountMemory(Ar));
		}

		if (InPartialBunch)
		{
			GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("InPartialBunch", InPartialBunch->CountMemory(Ar));
		}
	}
}

bool UChannel::ReceivedAcks(EChannelCloseReason& OutCloseReason)
{
	check(Connection->Channels[ChIndex]==this);

	/*
	// Verify in sequence.
	for( FOutBunch* Out=OutRec; Out && Out->Next; Out=Out->Next )
		check(Out->Next->ChSequence>Out->ChSequence);
	*/

	// Release all acknowledged outgoing queued bunches.
	bool bCleanup = false;
	EChannelCloseReason CloseReason = EChannelCloseReason::Destroyed;

	while( OutRec && OutRec->ReceivedAck )
	{
		if (OutRec->bOpen)
		{
			bool OpenFinished = true;
			if (OutRec->bPartial)
			{
				// Partial open bunches: check that all open bunches have been ACKd before trashing them
				FOutBunch*	OpenBunch = OutRec;
				while (OpenBunch)
				{
					UE_LOGF(LogNet, VeryVerbose, "   Channel %i open partials %d ackd %d final %d ", ChIndex, OpenBunch->PacketId, OpenBunch->ReceivedAck, OpenBunch->bPartialFinal);

					if (!OpenBunch->ReceivedAck)
					{
						OpenFinished = false;
						break;
					}
					if(OpenBunch->bPartialFinal)
					{
						break;
					}

					OpenBunch = OpenBunch->Next;
				}
			}
			if (OpenFinished)
			{
				UE_LOGF(LogNet, VeryVerbose, "Channel %i is fully acked. PacketID: %d", ChIndex, OutRec->PacketId );
				OpenAcked = 1;
			}
			else
			{
				// Don't delete this bunch yet until all open bunches are Ackd.
				break;
			}
		}

		bCleanup = bCleanup || !!OutRec->bClose;

		if (OutRec->bClose)
		{
			CloseReason = OutRec->CloseReason;
		}

		FOutBunch* Release = OutRec;
		OutRec = OutRec->Next;
		delete Release;
		NumOutRec--;
	}

	if (OpenTemporary && OpenAcked)
	{
		// If this was a temporary channel we can close it now as we do not expect the other side to immediately close the temporary channel
		UE_LOGF(LogNetDormancy, Verbose, "ReceivedAcks: Cleaning up after close acked. CloseReason: %ls %ls", LexToString(CloseReason), *Describe());

		check(!OutRec);
		ConditionalCleanUp(false, CloseReason);
	}
	else if (bCleanup)
	{
		// If a close has been acknowledged in sequence, we're done.
		// We leave it to the caller to cleanup non-temporary channels since we want to process incoming data on the channel contained in the same packet.
		UE_LOGF(LogNetDormancy, Verbose, "ReceivedAcks: Channel queued for cleaning up after close acked. CloseReason: %ls %ls", LexToString(CloseReason), *Describe());

		check(!OutRec);
		OutCloseReason = CloseReason;
		return true;
	}

	return false;
}

void UChannel::ReceivedAcks()
{
	EChannelCloseReason CloseReason;
	if (ReceivedAcks(CloseReason))
	{
		ConditionalCleanUp(false, CloseReason);
	}
}

void UChannel::Tick()
{
	checkSlow(Connection->Channels[ChIndex]==this);
	if (bPendingDormancy && ReadyForDormancy())
	{
		BecomeDormant();
	}
}


void UChannel::AssertInSequenced()
{
#if DO_CHECK
	// Verify that buffer is in order with no duplicates.
	for( FInBunch* In=InRec; In && In->Next; In=In->Next )
		check(In->Next->ChSequence>In->ChSequence);
#endif
}


bool UChannel::ReceivedSequencedBunch( FInBunch& Bunch )
{
	SCOPED_NAMED_EVENT(UChannel_ReceivedSequencedBunch, FColor::Green);
	// Handle a regular bunch.
	if ( !Closing )
	{
		ReceivedBunch( Bunch );
	}

	// We have fully received the bunch, so process it.
	if( Bunch.bClose )
	{
		Dormant = (Bunch.CloseReason == EChannelCloseReason::Dormancy);

		// Handle a close-notify.
		if( InRec )
		{
			ensureMsgf(false, TEXT("Close Anomaly %i / %i"), Bunch.ChSequence, InRec->ChSequence );
		}

		if ( ChIndex == 0 )
		{
			UE_LOGF(LogNet, Log, "UChannel::ReceivedSequencedBunch: Bunch.bClose == true. ChIndex == 0. Calling ConditionalCleanUp." );
		}

		UE_LOGF(LogNetTraffic, Log, "UChannel::ReceivedSequencedBunch: Bunch.bClose == true. Calling ConditionalCleanUp. ChIndex: %i Reason: %ls", ChIndex, LexToString(Bunch.CloseReason));

		ConditionalCleanUp(false, Bunch.CloseReason);
		return true;
	}
	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UChannel::ReceivedRawBunch( FInBunch & Bunch, bool & bOutSkipAck )
{
	SCOPE_CYCLE_COUNTER(Stat_ChannelReceivedRawBunch);

	SCOPED_NAMED_EVENT(UChannel_ReceivedRawBunch, FColor::Green);
	// Immediately consume the NetGUID portion of this bunch, regardless if it is partial or reliable.
	// NOTE - For replays, we do this even earlier, to try and load this as soon as possible, in case there is an issue creating the channel
	// If a replay fails to create a channel, we want to salvage as much as possible
	if ( Bunch.bHasPackageMapExports && !Connection->IsInternalAck() )
	{
		Cast<UPackageMapClient>( Connection->PackageMap )->ReceiveNetGUIDBunch( Bunch );

		if ( Bunch.IsError() )
		{
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ReceivedNetGUIDBunchFail);

			UE_LOGF( LogNetTraffic, Error, "UChannel::ReceivedRawBunch: Bunch.IsError() after ReceiveNetGUIDBunch. ChIndex: %i", ChIndex );

			return;
		}
	}

	if ( Connection->IsInternalAck() && Broken )
	{
		if (Bunch.bClose && (!Bunch.bPartial || Bunch.bPartialFinal))
		{
			UE_LOGF(LogNetTraffic, Log, "Replay handling a close bunch for a broken channel: %ls", *Bunch.ToString());

			Dormant = (Bunch.CloseReason == EChannelCloseReason::Dormancy);
			ConditionalCleanUp(true, Bunch.CloseReason);
		}

		return;
	}

	check(Connection->Channels[ChIndex]==this);

	if ( Bunch.bReliable && Bunch.ChSequence != Connection->InReliable[ChIndex] + 1 )
	{
		// We shouldn't hit this path on 100% reliable connections
		check( !Connection->IsInternalAck() );
		// If this bunch has a dependency on a previous unreceived bunch, buffer it.
		checkSlow(!Bunch.bOpen);

		// Verify that UConnection::ReceivedPacket has passed us a valid bunch.
		check(Bunch.ChSequence>Connection->InReliable[ChIndex]);

		// Pessimistic NAK so that the other side doesn't think we have processed the export
		// In practice, this would only happen if we split an export bunch into multiple bunches
		if (Bunch.bPartialCustomExportsFinal)
		{
			UE_LOGF(LogNetPartialBunch, Warning, "New Pessimistic Nak Performed on %ls", *Bunch.ToString());
			bOutSkipAck = true;
		}

		// Find the place for this item, sorted in sequence.
		UE_LOGF(LogNetTraffic, Log, "      Queuing bunch with unreceived dependency: %d / %d", Bunch.ChSequence, Connection->InReliable[ChIndex]+1 );
		FInBunch** InPtr;
		for( InPtr=&InRec; *InPtr; InPtr=&(*InPtr)->Next )
		{
			if( Bunch.ChSequence==(*InPtr)->ChSequence )
			{
				// Already queued.
				return;
			}
			else if( Bunch.ChSequence<(*InPtr)->ChSequence )
			{
				// Stick before this one.
				break;
			}
		}

		FInBunch* New = new FInBunch(Bunch);
		New->Next     = *InPtr;
		*InPtr        = New;
		NumInRec++;
		Connection->GetDriver()->GetMetrics()->SetMaxInt(UE::Net::Metric::IncomingReliableMessageQueueMaxSize, NumInRec);

		if ( NumInRec >= RELIABLE_BUFFER )
		{
			Bunch.SetError();
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::MaxReliableExceeded);

			UE_LOGF( LogNetTraffic, Error, "UChannel::ReceivedRawBunch: Too many reliable messages queued up" );

			return;
		}

		checkSlow(NumInRec<=RELIABLE_BUFFER);
		//AssertInSequenced();
	}
	else
	{
		bool bDeleted = ReceivedNextBunch( Bunch, bOutSkipAck );

		if ( Bunch.IsError() )
		{
			UE_LOGF( LogNetTraffic, Error, "UChannel::ReceivedRawBunch: Bunch.IsError() after ReceivedNextBunch 1" );

			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ReceivedNextBunchFail);

			return;
		}

		if (bDeleted)
		{
			return;
		}

		// Dispatch any waiting bunches.
		while( InRec )
		{
			// We shouldn't hit this path on 100% reliable connections
			check( !Connection->IsInternalAck() );

			if( InRec->ChSequence!=Connection->InReliable[ChIndex]+1 )
				break;
			UE_LOGF(LogNetTraffic, Log, "      Channel %d Unleashing queued bunch", ChIndex );
			FInBunch* Release = InRec;
			InRec = InRec->Next;
			NumInRec--;

			// Removed from InRec, clear pointer to the rest of the chain before processing
			Release->Next = nullptr;

			// Just keep a local copy of the bSkipAck flag, since these have already been acked and it doesn't make sense on this context
			// Definitely want to warn when this happens, since it's really not possible
			bool bLocalSkipAck = false;

			bDeleted = ReceivedNextBunch( *Release, bLocalSkipAck );

			if ( bLocalSkipAck )
			{
				UE_LOGF( LogNetTraffic, Warning, "UChannel::ReceivedRawBunch: bLocalSkipAck == true for already acked packet" );
			}

			if ( Bunch.IsError() )
			{
				UE_LOGF( LogNetTraffic, Error, "UChannel::ReceivedRawBunch: Bunch.IsError() after ReceivedNextBunch 2" );

				AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ReceivedNextBunchQueueFail);

				return;
			}

			delete Release;
			if (bDeleted)
			{
				return;
			}
			//AssertInSequenced();
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

static void LogPartialBunch(const FString& Label, const FInBunch& UberBunch, const FInBunch& PartialBunch)
{
	// Don't want to call appMemcrc unless we need to
	if (UE_LOG_ACTIVE(LogNetPartialBunch, Verbose))
	{
		UE_LOG(LogNetPartialBunch, Verbose, TEXT("%s Channel: %d ChSequence: %d. NumBits Total: %" INT64_FMT ". NumBytes Left: %" INT64_FMT ". Rel: %d CRC 0x%X"), *Label, PartialBunch.ChIndex, PartialBunch.ChSequence, PartialBunch.GetNumBits(), UberBunch.GetBytesLeft(), UberBunch.bReliable, FCrc::MemCrc_DEPRECATED(PartialBunch.GetData(), PartialBunch.GetNumBytes()));
	}
}

bool UChannel::ReceivedNextBunch( FInBunch & Bunch, bool & bOutSkipAck )
{
	// We received the next bunch. Basically at this point:
	//	-We know this is in order if reliable
	//	-We dont know if this is partial or not
	// If its not a partial bunch, of it completes a partial bunch, we can call ReceivedSequencedBunch to actually handle it

	// Note this bunch's retirement.
	if ( Bunch.bReliable )
	{
		// Reliables should be ordered properly at this point
		check( Bunch.ChSequence == Connection->InReliable[Bunch.ChIndex] + 1 );

		Connection->InReliable[Bunch.ChIndex] = Bunch.ChSequence;
	}

	FInBunch* HandleBunch = &Bunch;
	if (Bunch.bPartial)
	{
		HandleBunch = NULL;
		if (Bunch.bPartialInitial)
		{
			// Create new InPartialBunch if this is the initial bunch of a new sequence.

			if (InPartialBunch != NULL)
			{
				if (!InPartialBunch->bPartialFinal)
				{
					if ( InPartialBunch->bReliable )
					{
						if ( Bunch.bReliable )
						{
							UE_LOGF(LogNetPartialBunch, Warning, "Reliable partial trying to destroy reliable partial 1. %ls", *Describe());

							Bunch.SetError();
							AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::PartialInitialReliableDestroy);

							return false;
						}
						UE_LOGF(LogNetPartialBunch, Log, "Unreliable partial trying to destroy reliable partial 1" );
						bOutSkipAck = true;
						return false;
					}

					// We didn't complete the last partial bunch - this isn't fatal since they can be unreliable, but may want to log it.
					UE_LOGF(LogNetPartialBunch, Verbose, "Incomplete partial bunch. Channel: %d ChSequence: %d", InPartialBunch->ChIndex, InPartialBunch->ChSequence);
				}

				delete InPartialBunch;
				InPartialBunch = NULL;
			}

			InPartialBunch = new FInBunch(Bunch, false);

			if ( InPartialBunch!=nullptr )
			{
				if ( !Bunch.bHasPackageMapExports && Bunch.GetBitsLeft() > 0 )
				{
					// If we're an extensions bunch, we're not byte aligned
					if ( !Bunch.bPartialCustomExportsFinal && Bunch.GetBitsLeft() % 8 != 0 )
					{
						UE_LOG(LogNetPartialBunch, Warning, TEXT("Corrupt partial bunch. Initial partial bunches are expected to be byte-aligned. BitsLeft = %" INT64_FMT ". %s"), Bunch.GetBitsLeft(), *Describe());

						Bunch.SetError();
						AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::PartialInitialNonByteAligned);

						return false;
					}

					InPartialBunch->AppendDataFromChecked( Bunch.GetDataPosChecked(), Bunch.GetBitsLeft() );

					LogPartialBunch(TEXT("Received new partial bunch."), Bunch, *InPartialBunch);
				}
				else
				{
					LogPartialBunch(TEXT("Received New partial bunch. It only contained NetGUIDs."), Bunch, *InPartialBunch);
				}
			}
		}
		else
		{
			// Merge in next partial bunch to InPartialBunch if:
			//	-We have a valid InPartialBunch
			//	-The current InPartialBunch wasn't already complete
			//  -ChSequence is next in partial sequence
			//	-Reliability flag matches

			bool bSequenceMatches = false;
			if (InPartialBunch)
			{
				const bool bReliableSequencesMatches = Bunch.ChSequence == InPartialBunch->ChSequence + 1;
				const bool bUnreliableSequenceMatches = bReliableSequencesMatches || (Bunch.ChSequence == InPartialBunch->ChSequence);

				// Unreliable partial bunches use the packet sequence, and since we can merge multiple bunches into a single packet,
				// it's perfectly legal for the ChSequence to match in this case.
				// Reliable partial bunches must be in consecutive order though
				bSequenceMatches = InPartialBunch->bReliable ? bReliableSequencesMatches : bUnreliableSequenceMatches;
			}

			if ( InPartialBunch && !InPartialBunch->bPartialFinal && bSequenceMatches && InPartialBunch->bReliable == Bunch.bReliable )
			{
				// Merge.
				UE_LOG(LogNetPartialBunch, Verbose, TEXT("Merging Partial Bunch: %" INT64_FMT " Bits."), Bunch.GetBitsLeft() );
				UE_LOGF(LogNetPartialBunch, VeryVerbose, " %ls", *Bunch.ToString() );

				if ( !Bunch.bHasPackageMapExports && Bunch.GetBitsLeft() > 0 )
				{
					InPartialBunch->AppendDataFromChecked( Bunch.GetDataPosChecked(), Bunch.GetBitsLeft() );
				}

				// Only the final partial bunch should ever be non byte aligned. This is enforced during partial bunch creation
				// This is to ensure fast copies/appending of partial bunches. The final partial bunch may be non byte aligned.
				if (!Bunch.bHasPackageMapExports && !Bunch.bPartialCustomExportsFinal && !Bunch.bPartialFinal && (Bunch.GetBitsLeft() % 8 != 0))
				{
					UE_LOG(LogNetPartialBunch, Warning, TEXT("Corrupt partial bunch. Non-final partial bunches are expected to be byte-aligned. bHasPackageMapExports = %d, bPartialFinal = %d, BitsLeft = %" INT64_FMT ". %s"),
						Bunch.bHasPackageMapExports ? 1 : 0, Bunch.bPartialFinal ? 1 : 0, Bunch.GetBitsLeft(), *Describe());

					Bunch.SetError();
					AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::PartialNonByteAligned);

					return false;
				}

				// Advance the sequence of the current partial bunch so we know what to expect next
				InPartialBunch->ChSequence = Bunch.ChSequence;

				if (Bunch.bPartialFinal)
				{
					LogPartialBunch(TEXT("Completed Partial Bunch."), Bunch, *InPartialBunch);

					if ( Bunch.bHasPackageMapExports )
					{
						// Shouldn't have these, they only go in initial partial export bunches
						UE_LOGF(LogNetPartialBunch, Warning, "Corrupt partial bunch. Final partial bunch has package map exports. %ls", *Describe());

						Bunch.SetError();
						AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::PartialFinalPackageMapExports);

						return false;
					}

					HandleBunch = InPartialBunch;

					InPartialBunch->bPartialFinal			= true;
					InPartialBunch->bClose					= Bunch.bClose;
					InPartialBunch->CloseReason				= Bunch.CloseReason;
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					InPartialBunch->bIsReplicationPaused	= Bunch.bIsReplicationPaused;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					InPartialBunch->bHasMustBeMappedGUIDs	= Bunch.bHasMustBeMappedGUIDs;
				}
				else
				{
					LogPartialBunch(TEXT("Received Partial Bunch."), Bunch, *InPartialBunch);
				}
			}
			else
			{
				// Merge problem - delete InPartialBunch. This is mainly so that in the unlikely chance that ChSequence wraps around, we wont merge two completely separate partial bunches.

				// We shouldn't hit this path on 100% reliable connections
				check( !Connection->IsInternalAck());

				bOutSkipAck = true;	// Don't ack the packet, since we didn't process the bunch

				if ( InPartialBunch && InPartialBunch->bReliable )
				{
					if ( Bunch.bReliable )
					{
						UE_LOGF(LogNetPartialBunch, Warning, "Reliable partial trying to destroy reliable partial 2. %ls", *Describe());

						Bunch.SetError();
						AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::PartialMergeReliableDestroy);

						return false;
					}

					UE_LOGF( LogNetPartialBunch, Log, "Unreliable partial trying to destroy reliable partial 2" );
					return false;
				}

				if (UE_LOG_ACTIVE(LogNetPartialBunch,Verbose)) // Don't want to call appMemcrc unless we need to
				{
					if (InPartialBunch)
					{
						LogPartialBunch(TEXT("Received Partial Bunch Out of Sequence."), Bunch, *InPartialBunch);
					}
					else
					{
						UE_LOGF(LogNetPartialBunch, Verbose, "Received Partial Bunch Out of Sequence when InPartialBunch was NULL!");
					}
				}

				if (InPartialBunch)
				{
					delete InPartialBunch;
					InPartialBunch = NULL;
				}


			}
		}

		if (InPartialBunch && IsBunchTooLarge(Connection, InPartialBunch))
		{
			UE_LOG(LogNetPartialBunch, Error, TEXT("Received a partial bunch exceeding max allowed size. BunchSize=%" INT64_FMT ", MaximumSize=%d Channel: %s"), InPartialBunch->GetNumBytes(), NetMaxConstructedPartialBunchSizeBytes, *Describe());

			Bunch.SetError();
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::PartialTooLarge);

			return false;
		}

		// This flag denotes that the previous partial data has now fully reassembled the exports data, and we should now reconstruct it
		// The reading of the exports data is guaranteed to occur in the same frame it's received (and therefore ack'd).  This is so an ack
		// guarantees that the export data has been read (so we can change our representation to a fully ack'd representation).
		if (!bOutSkipAck && Bunch.bPartialCustomExportsFinal)
		{
			UE_LOGF(LogNetPartialBunch, Verbose, "Partial CustomExports Bunch (ChSequence %d PacketId %d) is being processed.", Bunch.ChSequence, Bunch.PacketId);

			ensureMsgf(!HandleBunch, TEXT("Code logic error:  CustomExports Bunch is going to be read as if it contains other data"));
			ensureMsgf(!Bunch.bPartialFinal, TEXT("CustomExports Bunch was also marked as the final bunch (therefore no data depended on this)"));
			ensureMsgf(InPartialBunch, TEXT("CustomExports Bunch was not properly merged into a Partial Bunch"));
			if (InPartialBunch && !Connection->IsInternalAck())
			{
				InPartialBunch->bPartialCustomExportsFinal = Bunch.bPartialCustomExportsFinal;
				CastChecked<UPackageMapClient>(Connection->PackageMap)->ReceiveCustomExportsBunch(*InPartialBunch);

				if (InPartialBunch->IsError())
				{
					AddToChainResultPtr(InPartialBunch->ExtendedError, ENetCloseResult::ReceivedNetGUIDBunchFail);
					UE_LOGF(LogNetTraffic, Error, "UChannel::ReceivedRawBunch: Bunch.IsError() after ReceiveCustomExportsBunch. ChIndex: %i", ChIndex);
					return false;
				}

				// Reset the partial bunch so that it only contains data (not the exports, which we just processed)...
				InPartialBunch->ResetData(*InPartialBunch, 0, InPartialBunch->GetNumBits());
				InPartialBunch->bPartialCustomExportsFinal = false;
			}
		}
	} // bPartial

	if ( HandleBunch != NULL )
	{
		const bool bBothSidesCanOpen = Connection->Driver && Connection->Driver->ChannelDefinitionMap[ChName].bServerOpen && Connection->Driver->ChannelDefinitionMap[ChName].bClientOpen;

		if ( HandleBunch->bOpen )
		{
			if ( !bBothSidesCanOpen )	// Voice channels can open from both side simultaneously, so ignore this logic until we resolve this
			{
				// If we opened the channel, we shouldn't be receiving bOpen commands from the other side
				checkf(!OpenedLocally, TEXT("Received channel open command for channel that was already opened locally. %s"), *Describe());

				if (!ensure( OpenPacketId.First == INDEX_NONE && OpenPacketId.Last == INDEX_NONE ))
				{
					UE_LOGF( LogNetTraffic, Error, "Received channel open command for channel that was already opened locally. %ls", *Describe() );

					Bunch.SetError();
					AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::AlreadyOpen);

					return false;
				}
			}

			// Remember the range.
			// In the case of a non partial, HandleBunch == Bunch
			// In the case of a partial, HandleBunch should == InPartialBunch, and Bunch should be the last bunch.
			OpenPacketId.First = HandleBunch->PacketId;
			OpenPacketId.Last = Bunch.PacketId;
			OpenAcked = true;

			UE_LOGF( LogNetTraffic, Verbose, "ReceivedNextBunch: Channel now fully open. ChIndex: %i, OpenPacketId.First: %i, OpenPacketId.Last: %i", ChIndex, OpenPacketId.First, OpenPacketId.Last );
		}

		if ( !bBothSidesCanOpen )	// Voice channels can open from both side simultaneously, so ignore this logic until we resolve this
		{
			// Don't process any packets until we've fully opened this channel
			// (unless we opened it locally, in which case it's safe to process packets)
			if ( !OpenedLocally && !OpenAcked )
			{
				if ( HandleBunch->bReliable )
				{
					UE_LOGF( LogNetTraffic, Error, "ReceivedNextBunch: Reliable bunch before channel was fully open. ChSequence: %i, OpenPacketId.First: %i, OpenPacketId.Last: %i, bPartial: %i, %ls", Bunch.ChSequence, OpenPacketId.First, OpenPacketId.Last, ( int32 )HandleBunch->bPartial, *Describe() );

					Bunch.SetError();
					AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ReliableBeforeOpen);

					return false;
				}

				if ( !ensure( !Connection->IsInternalAck()) )
				{
					// Shouldn't be possible for 100% reliable connections
					Broken = 1;
					return false;
				}

				// Don't ack this packet (since we won't process all of it)
				bOutSkipAck = true;

				UE_LOGF( LogNetTraffic, Verbose, "ReceivedNextBunch: Skipping bunch since channel isn't fully open. ChIndex: %i", ChIndex );
				return false;
			}

			// At this point, we should have the open packet range
			// This is because if we opened the channel locally, we set it immediately when we sent the first bOpen bunch
			// If we opened it from a remote connection, then we shouldn't be processing any packets until it's fully opened (which is handled above)
			check( OpenPacketId.First != INDEX_NONE );
			check( OpenPacketId.Last != INDEX_NONE );
		}

		// Receive it in sequence.
		return ReceivedSequencedBunch( *HandleBunch );
	}

	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UChannel::AppendExportBunches( TArray<FOutBunch *>& OutExportBunches )
{
	UPackageMapClient * PackageMapClient = CastChecked< UPackageMapClient >( Connection->PackageMap );

	// Let the package map add any outgoing bunches it needs to send
	PackageMapClient->AppendExportBunches( OutExportBunches );
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TArray<FOutBunch*> UChannel::GetAdditionalRequiredBunches(const FOutBunch& OutgoingBunch, EChannelGetAdditionalRequiredBunchesFlags Flags)
{
	UPackageMapClient* PackageMapClient = CastChecked< UPackageMapClient >(Connection->PackageMap);
	return PackageMapClient->GetAdditionalRequiredBunches(OutgoingBunch, Flags);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UChannel::AppendMustBeMappedGuids( FOutBunch* Bunch )
{
	UPackageMapClient * PackageMapClient = CastChecked< UPackageMapClient >( Connection->PackageMap );

	TArray< FNetworkGUID >& MustBeMappedGuidsInLastBunch = PackageMapClient->GetMustBeMappedGuidsInLastBunch();

	if ( MustBeMappedGuidsInLastBunch.Num() > 0 )
	{
		// Rewrite the bunch with the unique guids in front
		FOutBunch TempBunch( *Bunch );

		Bunch->Reset();

		// Write all the guids out
		uint16 NumMustBeMappedGUIDs = MustBeMappedGuidsInLastBunch.Num();
		*Bunch << NumMustBeMappedGUIDs;

		for (FNetworkGUID& NetGUID : MustBeMappedGuidsInLastBunch)
		{
			*Bunch << NetGUID;
		}

		NETWORK_PROFILER(GNetworkProfiler.TrackMustBeMappedGuids(NumMustBeMappedGUIDs, Bunch->GetNumBits(), Connection));

		// Append the original bunch data at the end
		Bunch->SerializeBits( TempBunch.GetData(), TempBunch.GetNumBits() );

		Bunch->bHasMustBeMappedGUIDs = 1;

		MustBeMappedGuidsInLastBunch.Reset();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const FString UActorChannel::ClassNetCacheSuffix = TEXT("_ClassNetCache");

UActorChannel::UActorChannel(const FObjectInitializer& ObjectInitializer)
	: UChannel(ObjectInitializer)
#if !UE_BUILD_SHIPPING
	, bBlockChannelFailure(false)
#endif
{
	ChName = NAME_Actor;
	bClearRecentActorRefs = true;
	bHoldQueuedExportBunchesAndGUIDs = false;
	QueuedCloseReason = EChannelCloseReason::Destroyed;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UActorChannel::~UActorChannel()
{

}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UActorChannel::AppendExportBunches( TArray<FOutBunch *>& OutExportBunches )
{
	ensureMsgf(false, TEXT("%hs is deprecated. Use GetAdditionalRequiredBunches"), __func__);

	if (bHoldQueuedExportBunchesAndGUIDs)
	{
		return;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Super::AppendExportBunches( OutExportBunches );
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// We don't want to append QueuedExportBunches to these bunches, since these were for queued RPC's, and we don't want to record RPC's during bResendAllDataSinceOpen
	if ( Connection->ResendAllDataState == EResendAllDataState::None )
	{
		// Let the profiler know about exported GUID bunches
		for ( const FOutBunch* ExportBunch : QueuedExportBunches )
		{
			if ( ExportBunch != nullptr )
			{
				NETWORK_PROFILER( GNetworkProfiler.TrackExportBunch( ExportBunch->GetNumBits(), Connection ) );
			}
		}

		if ( QueuedExportBunches.Num() )
		{
			OutExportBunches.Append( QueuedExportBunches );
			QueuedExportBunches.Empty();
		}
	}
}

TArray<FOutBunch*> UActorChannel::GetAdditionalRequiredBunches(const FOutBunch& OutgoingBunch, EChannelGetAdditionalRequiredBunchesFlags Flags)
{
	TArray<FOutBunch*> AdditionalBunches;

	// Tell super to skip appending NetGUID export bunches.
	// This is used from fastpath to disallow NetGUID exports but since we now allow "stable" exports we need to get the additional bunches for that type of exportdata.
	Flags |= bHoldQueuedExportBunchesAndGUIDs ? EChannelGetAdditionalRequiredBunchesFlags::SkipNetGUIDExports : EChannelGetAdditionalRequiredBunchesFlags::None;

	AdditionalBunches = Super::GetAdditionalRequiredBunches(OutgoingBunch, Flags);

	// We don't want to append QueuedExportBunches to these bunches, since these were for queued RPC's, and we don't want to record RPC's during bResendAllDataSinceOpen
	if (Connection->ResendAllDataState == EResendAllDataState::None)
	{
		// Let the profiler know about exported GUID bunches
		for (const FOutBunch* ExportBunch : QueuedExportBunches)
		{
			if (ExportBunch != nullptr)
			{
				NETWORK_PROFILER(GNetworkProfiler.TrackExportBunch(ExportBunch->GetNumBits(), Connection));
			}
		}

		if (QueuedExportBunches.Num())
		{
			AdditionalBunches.Append(QueuedExportBunches);
			QueuedExportBunches.Empty();
		}
	}

	return AdditionalBunches;
}

void UActorChannel::AppendMustBeMappedGuids( FOutBunch* Bunch )
{
	if (bHoldQueuedExportBunchesAndGUIDs)
	{
		return;
	}

	// We don't want to append QueuedMustBeMappedGuidsInLastBunch to these bunches, since these were for queued RPC's, and we don't want to record RPC's during bResendAllDataSinceOpen
	if ( Connection->ResendAllDataState == EResendAllDataState::None )
	{
		if ( QueuedMustBeMappedGuidsInLastBunch.Num() > 0 )
		{
			// Just add our list to the main list on package map so we can re-use the code in UChannel to add them all together
			UPackageMapClient * PackageMapClient = CastChecked< UPackageMapClient >( Connection->PackageMap );

			PackageMapClient->GetMustBeMappedGuidsInLastBunch().Append( QueuedMustBeMappedGuidsInLastBunch );

			QueuedMustBeMappedGuidsInLastBunch.Reset();
		}
	}

	// Actually add them to the bunch
	// NOTE - We do this LAST since we want to capture the append that happened above
	Super::AppendMustBeMappedGuids( Bunch );
}

FPacketIdRange UChannel::SendBunch( FOutBunch* Bunch, bool Merge )
{
	LLM_SCOPE_BYTAG(NetChannel);

	if (!ensure(ChIndex != -1))
	{
		// Client "closing" but still processing bunches. Client->Server RPCs should avoid calling this, but perhaps more code needs to check this condition.
		return FPacketIdRange(INDEX_NONE);
	}

	if (!ensureMsgf(!IsBunchTooLarge(Connection, Bunch), TEXT("Attempted to send bunch exceeding max allowed size. BunchSize=%" INT64_FMT ", MaximumSize=%d Channel: %s"), Bunch->GetNumBytes(), NetMaxConstructedPartialBunchSizeBytes, *Describe()))
	{
		UE_LOG(LogNetPartialBunch, Error, TEXT("Attempted to send bunch exceeding max allowed size. BunchSize=%" INT64_FMT ", MaximumSize=%d Channel: %s"), Bunch->GetNumBytes(), NetMaxConstructedPartialBunchSizeBytes, *Describe());
		Bunch->SetError();
		return FPacketIdRange(INDEX_NONE);
	}

	check(!Closing);
	checkf(Connection->Channels[ChIndex]==this, TEXT("This: %s, Connection->Channels[ChIndex]: %s"), *Describe(), Connection->Channels[ChIndex] ? *Connection->Channels[ChIndex]->Describe() : TEXT("Null"));
	check(!Bunch->IsError());
	check( !Bunch->bHasPackageMapExports );

	// Set bunch flags.

	const bool bDormancyClose = Bunch->bClose && (Bunch->CloseReason == EChannelCloseReason::Dormancy);

	if (OpenedLocally && ((OpenPacketId.First == INDEX_NONE) || ((Connection->ResendAllDataState != EResendAllDataState::None) && !bDormancyClose)))
	{
		bool bOpenBunch = true;

		if (Connection->ResendAllDataState == EResendAllDataState::SinceCheckpoint)
		{
			bOpenBunch = !bOpenedForCheckpoint;
			bOpenedForCheckpoint = true;
		}

		if (bOpenBunch)
		{
			Bunch->bOpen = 1;
			OpenTemporary = !Bunch->bReliable;
		}
	}

	// If channel was opened temporarily, we are never allowed to send reliable packets on it.
	check(!OpenTemporary || !Bunch->bReliable);

	// This is the max number of bits we can have in a single bunch
	const int64 MAX_SINGLE_BUNCH_SIZE_BITS  = Connection->GetMaxSingleBunchSizeBits();

	// Max bytes we'll put in a partial bunch
	const int64 MAX_SINGLE_BUNCH_SIZE_BYTES = MAX_SINGLE_BUNCH_SIZE_BITS / 8;

	// Max bits will put in a partial bunch (byte aligned, we dont want to deal with partial bytes in the partial bunches)
	const int64 MAX_PARTIAL_BUNCH_SIZE_BITS = MAX_SINGLE_BUNCH_SIZE_BYTES * 8;

	TArray<FOutBunch*>& OutgoingBunches = Connection->GetOutgoingBunches();
	OutgoingBunches.Reset();

	// Add any export bunches
	// Replay connections will manage export bunches separately.
	if (!Connection->IsInternalAck())
	{
		OutgoingBunches = GetAdditionalRequiredBunches(*Bunch);
	}

	if ( OutgoingBunches.Num() )
	{
		// Don't merge if we are exporting guid's
		// We can't be for sure if the last bunch has exported guids as well, so this just simplifies things
		Merge = false;
	}

	if ( Connection->Driver->IsServer() )
	{
		// This is a bit special, currently we report this is at the end of bunch event though AppendMustBeMappedGuids rewrites the entire bunch
		UE_NET_TRACE_SCOPE(MustBeMappedGuids_IsAtStartOfBunch, *Bunch, GetTraceCollector(*Bunch), ENetTraceVerbosity::Trace);

		// Append any "must be mapped" guids to front of bunch from the packagemap
		AppendMustBeMappedGuids( Bunch );

		if ( Bunch->bHasMustBeMappedGUIDs )
		{
			// We can't merge with this, since we need all the unique static guids in the front
			Merge = false;
		}
	}

	//-----------------------------------------------------
	// Contemplate merging.
	//-----------------------------------------------------
	int32 PreExistingBits = 0;
	FOutBunch* OutBunch = NULL;
	if
	(	Merge
	&&	Connection->LastOut.ChIndex == Bunch->ChIndex
	&&	Connection->LastOut.bReliable == Bunch->bReliable	// Don't merge bunches of different reliability, since for example a reliable RPC can cause a bunch with properties to become reliable, introducing unnecessary latency for the properties.
	&&	Connection->AllowMerge
	&&	Connection->LastEnd.GetNumBits()
	&&	Connection->LastEnd.GetNumBits()==Connection->SendBuffer.GetNumBits()
	&&	Connection->LastOut.GetNumBits() + Bunch->GetNumBits() <= MAX_SINGLE_BUNCH_SIZE_BITS )
	{
		// Merge.
		check(!Connection->LastOut.IsError());
		PreExistingBits = Connection->LastOut.GetNumBits();
		Connection->LastOut.SerializeBits( Bunch->GetData(), Bunch->GetNumBits() );
		Connection->LastOut.bOpen     |= Bunch->bOpen;
		Connection->LastOut.bClose    |= Bunch->bClose;
#if UE_NET_TRACE_ENABLED
		SetTraceCollector(Connection->LastOut, GetTraceCollector(*Bunch));
		SetTraceCollector(*Bunch, nullptr);
#endif

		OutBunch                       = Connection->LastOutBunch;
		Bunch                          = &Connection->LastOut;
		check(!Bunch->IsError());
		Connection->PopLastStart();
		Connection->Driver->OutBunches--;
	}

	//-----------------------------------------------------
	// Possibly split large bunch into list of smaller partial bunches
	//-----------------------------------------------------
	if( Bunch->GetNumBits() > MAX_SINGLE_BUNCH_SIZE_BITS )
	{
		uint8 *data = Bunch->GetData();
		int64 bitsLeft = Bunch->GetNumBits();
		Merge = false;

		while(bitsLeft > 0)
		{
			FOutBunch * PartialBunch = new FOutBunch(this, false);
			int64 bitsThisBunch = FMath::Min<int64>(bitsLeft, MAX_PARTIAL_BUNCH_SIZE_BITS);
			PartialBunch->SerializeBits(data, bitsThisBunch);

#if UE_NET_TRACE_ENABLED
			// Attach tracecollector of split bunch to first partial bunch
			SetTraceCollector(*PartialBunch, GetTraceCollector(*Bunch));
			SetTraceCollector(*Bunch, nullptr);
#endif

			OutgoingBunches.Add(PartialBunch);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			PartialBunch->DebugString = FString::Printf(TEXT("Partial[%d]: %s"), OutgoingBunches.Num(), *Bunch->DebugString);
#endif

			bitsLeft -= bitsThisBunch;
			data += (bitsThisBunch >> 3);

			UE_LOG(LogNetPartialBunch, Log, TEXT("	Making partial bunch from content bunch. bitsThisBunch: %" INT64_FMT " bitsLeft: %" INT64_FMT), bitsThisBunch, bitsLeft );

			ensure(bitsLeft == 0 || bitsThisBunch % 8 == 0); // Byte aligned or it was the last bunch
		}
	}
	else
	{
		OutgoingBunches.Add(Bunch);
	}

	//-----------------------------------------------------
	// Send all the bunches we need to
	//	Note: this is done all at once. We could queue this up somewhere else before sending to Out.
	//-----------------------------------------------------
	FPacketIdRange PacketIdRange;

	const bool bOverflowsReliable = (NumOutRec + OutgoingBunches.Num() >= RELIABLE_BUFFER + Bunch->bClose);

	if ((GCVarNetPartialBunchReliableThreshold > 0) && (OutgoingBunches.Num() >= GCVarNetPartialBunchReliableThreshold) && !Connection->IsInternalAck())
	{
		if (!bOverflowsReliable)
		{
			UE_LOGF(LogNetPartialBunch, Log, "	OutgoingBunches.Num (%d) exceeds reliable threashold (%d). Making bunches reliable. Property replication will be paused on this channel until these are ACK'd.", OutgoingBunches.Num(), GCVarNetPartialBunchReliableThreshold);
			Bunch->bReliable = true;
			bPausedUntilReliableACK = true;
		}
		else
		{
			// The threshold was hit, but making these reliable would overflow the reliable buffer. This is a problem: there is just too much data.
			UE_LOGF(LogNetPartialBunch, Warning, "	OutgoingBunches.Num (%d) exceeds reliable threashold (%d) but this would overflow the reliable buffer! Consider sending less stuff. Channel: %ls", OutgoingBunches.Num(), GCVarNetPartialBunchReliableThreshold, *Describe());
		}
	}

	if (Bunch->bReliable && bOverflowsReliable)
	{
		UE_LOGF(LogNetPartialBunch, Warning, "SendBunch: Reliable partial bunch overflows reliable buffer! %ls", *Describe() );
		UE_LOGF(LogNetPartialBunch, Warning, "   Num OutgoingBunches: %d. NumOutRec: %d", OutgoingBunches.Num(), NumOutRec );
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PrintReliableBunchBuffer();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Bail out, we can't recover from this (without increasing RELIABLE_BUFFER)
		FString ErrorMsg = NSLOCTEXT("NetworkErrors", "ClientReliableBufferOverflow", "Outgoing reliable buffer overflow").ToString();

		Connection->SendCloseReason(ENetCloseResult::ReliableBufferOverflow);
		FNetControlMessage<NMT_Failure>::Send(Connection, ErrorMsg);
		Connection->FlushNet(true);
		Connection->Close(ENetCloseResult::ReliableBufferOverflow);

		return PacketIdRange;
	}

	UE_CLOGF((OutgoingBunches.Num() > 1), LogNetPartialBunch, Log, "Sending %d Bunches. Channel: %d %ls", OutgoingBunches.Num(), Bunch->ChIndex, *Describe());
	for( int32 PartialNum = 0; PartialNum < OutgoingBunches.Num(); ++PartialNum)
	{
		FOutBunch * NextBunch = OutgoingBunches[PartialNum];

		NextBunch->bReliable = Bunch->bReliable;
		NextBunch->bOpen = Bunch->bOpen;
		NextBunch->bClose = Bunch->bClose;
		NextBunch->CloseReason = Bunch->CloseReason;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NextBunch->bIsReplicationPaused = Bunch->bIsReplicationPaused;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		NextBunch->ChIndex = Bunch->ChIndex;
		NextBunch->ChName = Bunch->ChName;

		if ( !(NextBunch->bHasPackageMapExports || NextBunch->bPartialCustomExportsFinal))
		{
			NextBunch->bHasMustBeMappedGUIDs |= Bunch->bHasMustBeMappedGUIDs;
		}

		if (OutgoingBunches.Num() > 1)
		{
			NextBunch->bPartial = 1;
			NextBunch->bPartialInitial = (PartialNum == 0 ? 1: 0);
			NextBunch->bPartialFinal = (PartialNum == OutgoingBunches.Num() - 1 ? 1: 0);
			NextBunch->bOpen &= (PartialNum == 0);											// Only the first bunch should have the bOpen bit set
			NextBunch->bClose = (Bunch->bClose && (OutgoingBunches.Num()-1 == PartialNum)); // Only last bunch should have bClose bit set
		}
		ensureMsgf(!NextBunch->bPartialCustomExportsFinal || NextBunch->bPartial, TEXT("Can't have PackageMapExtensions without being part of a partial bunch"));

		FOutBunch *ThisOutBunch = PrepBunch(NextBunch, OutBunch, Merge); // This handles queuing reliable bunches into the ack list

		if (UE_LOG_ACTIVE(LogNetPartialBunch,Verbose) && (OutgoingBunches.Num() > 1)) // Don't want to call appMemcrc unless we need to
		{
			UE_LOG(LogNetPartialBunch, Verbose, TEXT("	Bunch[%d]: Bytes: %" INT64_FMT " Bits: %" INT64_FMT " ChSequence: %d NetGUIDs: %d CustomExports: %d 0x%X"), PartialNum, ThisOutBunch->GetNumBytes(), ThisOutBunch->GetNumBits(), ThisOutBunch->ChSequence, ThisOutBunch->bHasPackageMapExports, ThisOutBunch->bPartialCustomExportsFinal, FCrc::MemCrc_DEPRECATED(ThisOutBunch->GetData(), ThisOutBunch->GetNumBytes()));
			UE_LOGF(LogNetPartialBunch, VeryVerbose, "	 %ls", *ThisOutBunch->GetDebugString());
		}

		// Update Packet Range
		int32 PacketId = SendRawBunch(ThisOutBunch, Merge, GetTraceCollector(*NextBunch));
		if (PartialNum == 0)
		{
			PacketIdRange = FPacketIdRange(PacketId);
		}
		else
		{
			PacketIdRange.Last = PacketId;
		}

		// Update channel sequence count.
		Connection->LastOut = *ThisOutBunch;
		Connection->LastEnd	= FBitWriterMark( Connection->SendBuffer );
	}

	// Update open range if necessary
	if (Bunch->bOpen && (Connection->ResendAllDataState == EResendAllDataState::None))
	{
		OpenPacketId = PacketIdRange;
	}

	// Destroy outgoing bunches now that they are sent, except the one that was passed into ::SendBunch
	//	This is because the one passed in ::SendBunch is the responsibility of the caller, the other bunches in OutgoingBunches
	//	were either allocated in this function for partial bunches, or taken from the package map, which expects us to destroy them.
	for (auto It = OutgoingBunches.CreateIterator(); It; ++It)
	{
		FOutBunch *DeleteBunch = *It;
		if (DeleteBunch != Bunch)
			delete DeleteBunch;
	}

	return PacketIdRange;
}

/** This returns a pointer to Bunch, but it may either be a direct pointer, or a pointer to a copied instance of it */

// OUtbunch is a bunch that was new'd by the network system or NULL. It should never be one created on the stack
FOutBunch* UChannel::PrepBunch(FOutBunch* Bunch, FOutBunch* OutBunch, bool Merge)
{
	if ( Connection->ResendAllDataState != EResendAllDataState::None )
	{
		return Bunch;
	}

	// Find outgoing bunch index.
	if( Bunch->bReliable )
	{
		// Find spot, which was guaranteed available by FOutBunch constructor.
		if( OutBunch==NULL )
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (!(NumOutRec<RELIABLE_BUFFER-1+Bunch->bClose))
			{
				UE_LOGF(LogNetTraffic, Warning, "PrepBunch: Reliable buffer overflow! %ls", *Describe());
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				PrintReliableBunchBuffer();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
#else
			check(NumOutRec<RELIABLE_BUFFER-1+Bunch->bClose);
#endif

			Bunch->Next	= NULL;
			Bunch->ChSequence = ++Connection->OutReliable[ChIndex];
			NumOutRec++;
			Connection->GetDriver()->GetMetrics()->SetMaxInt(UE::Net::Metric::OutgoingReliableMessageQueueMaxSize, NumOutRec);
			OutBunch = new FOutBunch(*Bunch);
			FOutBunch** OutLink = &OutRec;
			while(*OutLink) // This was rewritten from a single-line for loop due to compiler complaining about empty body for loops (-Wempty-body)
			{
				OutLink=&(*OutLink)->Next;
			}
			*OutLink = OutBunch;
		}
		else
		{
			Bunch->Next = OutBunch->Next;
			*OutBunch = *Bunch;
		}
		Connection->LastOutBunch = OutBunch;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (CVarNetReliableDebug.GetValueOnAnyThread() == 1)
		{
			UE_LOGF(LogNetTraffic, Warning, "%ls. Reliable: %ls", *Describe(), *Bunch->DebugString);
		}
		if (CVarNetReliableDebug.GetValueOnAnyThread() == 2)
		{
			UE_LOGF(LogNetTraffic, Warning, "%ls. Reliable: %ls", *Describe(), *Bunch->DebugString);
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			PrintReliableBunchBuffer();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			UE_LOGF(LogNetTraffic, Warning, "");
		}
#endif

	}
	else
	{
		OutBunch = Bunch;
		Connection->LastOutBunch = NULL;//warning: Complex code, don't mess with this!
	}

	return OutBunch;
}

int32 UChannel::SendRawBunch(FOutBunch* OutBunch, bool Merge, const FNetTraceCollector* Collector)
{
	// Sending for checkpoints may need to send an open bunch if the actor went dormant, so allow the OpenPacketId to be set

	// Send the raw bunch.
	OutBunch->ReceivedAck = 0;
	int32 PacketId = Connection->SendRawBunch(*OutBunch, Merge, Collector);
	if( OpenPacketId.First==INDEX_NONE && OpenedLocally )
	{
		OpenPacketId = FPacketIdRange(PacketId);
	}

	if( OutBunch->bClose )
	{
		SetClosingFlag();
	}

	return PacketId;
}

FString UChannel::Describe()
{
	return FString::Printf(TEXT("[UChannel] ChIndex: %d, Closing: %d %s"), ChIndex, (int32)Closing, Connection ? *Connection->Describe() : TEXT( "NULL CONNECTION" ));
}

int32 UChannel::IsNetReady( bool Saturate )
{
	// If saturation allowed, ignore queued byte count.
	if (NumOutRec >= RELIABLE_BUFFER - 1)
	{
		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Connection->IsNetReady(Saturate);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UChannel::IsNetReady() const
{
	// If saturation allowed, ignore queued byte count.
	if( NumOutRec>=RELIABLE_BUFFER-1 )
	{
		return false;
	}
	return Connection->IsNetReady();
}

void UChannel::ReceivedAck( int32 AckPacketId )
{
	// Do nothing. Most channels deal with this in Tick().
}

void UChannel::ReceivedNak( int32 NakPacketId )
{
	for( FOutBunch* Out=OutRec; Out; Out=Out->Next )
	{
		// Retransmit reliable bunches in the lost packet.
		if( Out->PacketId==NakPacketId && !Out->ReceivedAck )
		{
			check(Out->bReliable);
			UE_LOGF(LogNetTraffic, Log, "      Channel %i nak); resending %i...", Out->ChIndex, Out->ChSequence );

			FNetTraceCollector* Collector = Connection->GetOutTraceCollector();
			if (Collector)
			{
				// Inject trace event for the resent bunch if tracing is enabled
				// The reason behind the complexity is that the outgoing sendbuffer migth be flushed during the call to SendRawBunch()
				FNetTraceCollector* TempCollector = UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace);
				if (TempCollector!=nullptr)
				{
					UE_NET_TRACE(ResendBunch, TempCollector, 0U, Out->GetNumBits(), ENetTraceVerbosity::Trace);
					Connection->SendRawBunch(*Out, 0, TempCollector);
					UE_NET_TRACE_DESTROY_COLLECTOR(TempCollector);
				}
			}
			else
			{
				Connection->SendRawBunch( *Out, 0 );
			}
		}
	}
}

void UChannel::PrintReliableBunchBuffer()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	for( FOutBunch* Out=OutRec; Out; Out=Out->Next )
	{
		UE_LOGF(LogNetTraffic, Warning, "Out: %ls", *Out->DebugString );
	}
	UE_LOGF(LogNetTraffic, Warning, "-------------------------\n");
#endif
}

void UChannel::AddedToChannelPool()
{
	check(!Connection);
	check(!InRec);
	check(!OutRec);
	check(!InPartialBunch);

	bPooled = true;

	OpenAcked = false;
	Closing = false;
	Dormant = false;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bIsReplicationPaused = false;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	OpenTemporary = false;
	Broken = false;
	bTornOff = false;
	bPendingDormancy = false;
	bIsInDormancyHysteresis = false;
	bPausedUntilReliableACK = false;
	SentClosingBunch = false;
	bOpenedForCheckpoint = false;
	ChIndex = 0;
	OpenedLocally = false;
	OpenPacketId = FPacketIdRange();
	NumInRec = 0;
	NumOutRec = 0;
}

bool UChannel::HasAcknowledgedAllReliableData() const
{
	return NumOutRec == 0;
}

/*-----------------------------------------------------------------------------
	UControlChannel implementation.
-----------------------------------------------------------------------------*/

const TCHAR* FNetControlMessageInfo::Names[256];
bool FNetControlMessageInfo::bInitialized = false;

// control channel message implementation
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Hello);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Welcome);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Upgrade);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Challenge);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Netspeed);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Login);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Failure);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Join);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(JoinSplit);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Skip);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Abort);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(PCSwap);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(ActorChannelFailure);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(DebugText);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(NetGUIDAssign);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(SecurityViolation);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(GameSpecific);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(EncryptionAck);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(DestructionInfo);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(CloseReason);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(NetPing);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(BeaconWelcome);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(BeaconJoin);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(BeaconAssignGUID);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(BeaconNetGUIDAck);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(IrisProtocolMismatch);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(IrisNetRefHandleError);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(IrisNetRefHandleErrorWithDiagnosticMessage);
IMPLEMENT_CONTROL_CHANNEL_MESSAGE(IrisProtocolMismatchWithCDOState);

void UControlChannel::Init( UNetConnection* InConnection, int32 InChannelIndex, EChannelCreateFlags CreateFlags )
{
	Super::Init( InConnection, InChannelIndex, CreateFlags );

	// If we are opened as a server connection, do the endian checking
	// The client assumes that the data will always have the correct byte order
		// Mark this channel as needing endianess determination
	bNeedsEndianInspection = !EnumHasAnyFlags(CreateFlags, EChannelCreateFlags::OpenedLocally);
}


bool UControlChannel::CheckEndianess(FInBunch& Bunch)
{
	// Assume the packet is bogus and the connection needs closing
	bool bConnectionOk = false;
	// Get pointers to the raw packet data
	const uint8* HelloMessage = Bunch.GetData();
	// Check for a packet that is big enough to look at (message ID (1 byte) + platform identifier (1 byte))
	if (Bunch.GetNumBytes() >= 2)
	{
		if (HelloMessage[0] == NMT_Hello)
		{
			// Get platform id
			uint8 OtherPlatformIsLittle = HelloMessage[1];
			checkSlow(OtherPlatformIsLittle == !!OtherPlatformIsLittle); // should just be zero or one, we use check slow because we don't want to crash in the wild if this is a bad value.
			uint8 IsLittleEndian = uint8(PLATFORM_LITTLE_ENDIAN);
			check(IsLittleEndian == !!IsLittleEndian); // should only be one or zero

			UE_LOGF(LogNet, Log, "Remote platform little endian=%d", int32(OtherPlatformIsLittle));
			UE_LOGF(LogNet, Log, "This platform little endian=%d", int32(IsLittleEndian));
			// Check whether the other platform needs byte swapping by
			// using the value sent in the packet. Note: we still validate it
			if (OtherPlatformIsLittle ^ IsLittleEndian)
			{
				// Client has opposite endianess so swap this bunch
				// and mark the connection as needing byte swapping
				Bunch.SetByteSwapping(true);
				Connection->bNeedsByteSwapping = true;
			}
			else
			{
				// Disable all swapping
				Bunch.SetByteSwapping(false);
				Connection->bNeedsByteSwapping = false;
			}
			// We parsed everything so keep the connection open
			bConnectionOk = true;
			bNeedsEndianInspection = false;
		}
	}
	return bConnectionOk;
}


void UControlChannel::ReceivedBunch( FInBunch& Bunch )
{
	check(!Closing);

	UE_NET_TRACE_SCOPE(ControlChannel, Bunch, Connection->GetInTraceCollector(), ENetTraceVerbosity::Trace);

	// If this is a new client connection inspect the raw packet for endianess
	if (Connection && bNeedsEndianInspection && !CheckEndianess(Bunch))
	{
		// Send close bunch and shutdown this connection
		UE_LOGF(LogNet, Warning, "UControlChannel::ReceivedBunch: NetConnection::Close() [%ls] [%ls] [%ls] from CheckEndianess(). FAILED. Closing connection.",
			Connection->Driver ? *Connection->Driver->NetDriverName.ToString() : TEXT("NULL"),
			Connection->PlayerController ? *Connection->PlayerController->GetName() : TEXT("NoPC"),
			Connection->OwningActor ? *Connection->OwningActor->GetName() : TEXT("No Owner"));

		Connection->Close(ENetCloseResult::ControlChannelEndianCheck);
		return;
	}

	bool bStopReadingBunch = false;
	// Process the packet
	while (!Bunch.AtEnd() && bStopReadingBunch == false && Connection != nullptr && Connection->GetConnectionState() != USOCK_Closed) // if the connection got closed, we don't care about the rest
	{
		uint8 MessageType = 0;
		Bunch << MessageType;
		if (Bunch.IsError())
		{
			break;
		}
		int32 Pos = Bunch.GetPosBits();

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(FNetControlMessageInfo::GetName(MessageType), Bunch, Connection ? Connection->GetInTraceCollector() : nullptr, ENetTraceVerbosity::Trace);

		// we handle Actor channel failure notifications ourselves
		if (MessageType == NMT_ActorChannelFailure)
		{
			if (Connection->Driver->ServerConnection == NULL)
			{
				int32 ChannelIndex;

				if (FNetControlMessage<NMT_ActorChannelFailure>::Receive(Bunch, ChannelIndex))
				{
					UE_LOGF(LogNet, Log, "Server connection received: %ls %d %ls", FNetControlMessageInfo::GetName(MessageType), ChannelIndex, *Describe());

					// Check if Channel index provided by client is valid and within range of channel on server
					if (ChannelIndex >= 0 && ChannelIndex < Connection->Channels.Num())
					{
						// Get the actor channel that the client provided as having failed
						UActorChannel* ActorChan = Cast<UActorChannel>(Connection->Channels[ChannelIndex]);

						UE_CLOGF(ActorChan != nullptr, LogNet, Log, "Actor channel failed: %ls", *ActorChan->Describe());

						// The channel and the actor attached to the channel exists on the server
						if (ActorChan != nullptr && ActorChan->Actor != nullptr)
						{
							// The channel that failed is the player controller thus the connection is broken
							if (ActorChan->Actor == Connection->PlayerController)
							{
								UE_LOGF(LogNet, Warning, "UControlChannel::ReceivedBunch: NetConnection::Close() [%ls] [%ls] [%ls] from failed to initialize the PlayerController channel. Closing connection.",
									Connection->Driver ? *Connection->Driver->NetDriverName.ToString() : TEXT("NULL"),
									Connection->PlayerController ? *Connection->PlayerController->GetName() : TEXT("NoPC"),
									Connection->OwningActor ? *Connection->OwningActor->GetName() : TEXT("No Owner"));

								Connection->Close(ENetCloseResult::ControlChannelPlayerChannelFail);
							}
							// The client has a PlayerController connection, report the actor failure to PlayerController
							else if (Connection->PlayerController != nullptr)
							{
								Connection->PlayerController->NotifyActorChannelFailure(ActorChan);
							}
							// The PlayerController connection doesn't exist for the client
							// but the client is reporting an actor channel failure that isn't the PlayerController
							else
							{
								//UE_LOGF(LogNet, Warning, "UControlChannel::ReceivedBunch: PlayerController doesn't exist for the client, but the client is reporting an actor channel failure that isn't the PlayerController.");
							}
						}
					}
					// The client is sending an actor channel failure message with an invalid
					// actor channel index
					// @PotentialDOSAttackDetection
					else
					{
						UE_LOGF(LogNet, Warning, "UControlChannel::ReceivedBunch: The client is sending an actor channel failure message with an invalid actor channel index.");
					}
				}
			}
		}
		else if (MessageType == NMT_GameSpecific)
		{
			// the most common Notify handlers do not support subclasses by default and so we redirect the game specific messaging to the GameInstance instead
			uint8 MessageByte;
			FString MessageStr;
			if (FNetControlMessage<NMT_GameSpecific>::Receive(Bunch, MessageByte, MessageStr))
			{
				UGameInstance* GameInstance = nullptr;
				if (Connection->Driver->World != NULL && Connection->Driver->World->GetGameInstance() != NULL)
				{
					GameInstance = Connection->Driver->World->GetGameInstance();
				}
				else
				{
					FWorldContext* Context = GEngine->GetWorldContextFromPendingNetGameNetDriver(Connection->Driver);
					GameInstance = Context ? Context->OwningGameInstance.Get() : nullptr;
				}
				if (GameInstance)
				{
					LLM_SCOPE_BYNAME(TEXT("Networking/GameControlMessage"));
					LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GameInstance->GetPackage(), ELLMTagSet::Assets);
					LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GameInstance->GetClass(), ELLMTagSet::AssetClasses);
					UE_TRACE_METADATA_SCOPE_ASSET(GameInstance, GameInstance->GetClass());
					GameInstance->HandleGameNetControlMessage(Connection, MessageByte, MessageStr);
				}
			}
		}
		else if (MessageType == NMT_SecurityViolation)
		{
			FString DebugMessage;
			if (FNetControlMessage<NMT_SecurityViolation>::Receive(Bunch, DebugMessage))
			{
				UE_LOGF(LogSecurity, Warning, "%ls: Closed: %ls", *Connection->RemoteAddressToString(), *DebugMessage);
				bStopReadingBunch = true;
			}
		}
		else if (MessageType == NMT_DestructionInfo)
		{
			if (!Connection->Driver->IsServer())
			{
				ReceiveDestructionInfo(Bunch);
			}
			else
			{
				UE_LOGF(LogNet, Warning, "UControlChannel::ReceivedBunch: Server received a NMT_DestructionInfo which is not supported. Closing connection.");
				Connection->Close(ENetCloseResult::ClientSentDestructionInfo);
			}
		}
		else if (MessageType == NMT_CloseReason)
		{
			FString CloseReasonList;

			if (FNetControlMessage<NMT_CloseReason>::Receive(Bunch, CloseReasonList) && !CloseReasonList.IsEmpty())
			{
				Connection->HandleReceiveCloseReason(CloseReasonList);
			}
		}
		else if (MessageType == NMT_NetPing)
		{
			ENetPingControlMessage NetPingMessageType;
			FString MessageStr;

			if (FNetControlMessage<NMT_NetPing>::Receive(Bunch, NetPingMessageType, MessageStr) &&
				NetPingMessageType <= ENetPingControlMessage::Max)
			{
				UE::Net::FNetPing::HandleNetPingControlMessage(Connection, NetPingMessageType, MessageStr);
			}
		}
		else if (MessageType == NMT_IrisProtocolMismatchWithCDOState)
		{
			uint64 NetRefHandleId = 0;
			TArray<uint8> ClientCDOStateBytes;
			if (FNetControlMessage<NMT_IrisProtocolMismatchWithCDOState>::Receive(Bunch, NetRefHandleId, ClientCDOStateBytes))
			{
				if (UReplicationSystem* IrisRepSystem = Connection->Driver->GetReplicationSystem())
				{
					IrisRepSystem->ReportProtocolMismatch(NetRefHandleId, Connection->GetConnectionHandle().GetParentConnectionId(), ClientCDOStateBytes);
				}
			}
		}
		else if (MessageType == NMT_IrisProtocolMismatch)
		{
			uint64 NetRefHandleId = 0;
			if (FNetControlMessage<NMT_IrisProtocolMismatch>::Receive(Bunch, NetRefHandleId))
			{
				if (UReplicationSystem* IrisRepSystem = Connection->Driver->GetReplicationSystem())
				{
					IrisRepSystem->ReportProtocolMismatch(NetRefHandleId, Connection->GetConnectionHandle().GetParentConnectionId(), TArray<uint8>());
				}
			}
		}
		else if (MessageType == NMT_IrisNetRefHandleError)
		{
			UE::Net::ENetRefHandleError NetRefHandleError = UE::Net::ENetRefHandleError::None;
			uint64 NetRefHandleId = 0;
			TArray<uint64> ExtraNetRefs;
			if (FNetControlMessage<NMT_IrisNetRefHandleError>::Receive(Bunch, NetRefHandleError, NetRefHandleId, ExtraNetRefs))
			{
				if (UReplicationSystem* IrisRepSystem = Connection->Driver->GetReplicationSystem())
				{
					IrisRepSystem->ReportErrorWithNetRefHandle(NetRefHandleError, NetRefHandleId, Connection->GetConnectionHandle().GetParentConnectionId(), ExtraNetRefs);
				}
			}
		}
		else if (MessageType == NMT_IrisNetRefHandleErrorWithDiagnosticMessage)
		{
			UE::Net::ENetRefHandleError NetRefHandleError = UE::Net::ENetRefHandleError::None;
			uint64 NetRefHandleId = 0;
			TArray<uint64> ExtraNetRefs;
			FString DiagnosticMessage;
			if (FNetControlMessage<NMT_IrisNetRefHandleErrorWithDiagnosticMessage>::Receive(Bunch, NetRefHandleError, NetRefHandleId, ExtraNetRefs, DiagnosticMessage))
			{
				if (UReplicationSystem* IrisRepSystem = Connection->Driver->GetReplicationSystem())
				{
					IrisRepSystem->ReportErrorWithNetRefHandle(NetRefHandleError, NetRefHandleId, Connection->GetConnectionHandle().GetParentConnectionId(), ExtraNetRefs, DiagnosticMessage);
				}
			}
		}
		else if (Connection->Driver->Notify != nullptr)
		{
			// Process control message on client/server connection
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
			if (Connection->Driver->ShouldReplicateTransactionally())
			{
				static FName TransactionalWorkName("UControlChannel::ReceivedBunch NotifyControlMessage");
				UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [this, MessageType, &Bunch]()
					{
						Connection->Driver->Notify->NotifyControlMessage(Connection, MessageType, Bunch);
					});
			}
			else
#endif
			{
				Connection->Driver->Notify->NotifyControlMessage(Connection, MessageType, Bunch);
			}
		}

		// if the message was not handled, eat it ourselves
		if (Pos == Bunch.GetPosBits() && !Bunch.IsError())
		{
			switch (MessageType)
			{
				case NMT_Hello:
					FNetControlMessage<NMT_Hello>::Discard(Bunch);
					break;
				case NMT_Welcome:
					FNetControlMessage<NMT_Welcome>::Discard(Bunch);
					break;
				case NMT_Upgrade:
					FNetControlMessage<NMT_Upgrade>::Discard(Bunch);
					break;
				case NMT_Challenge:
					FNetControlMessage<NMT_Challenge>::Discard(Bunch);
					break;
				case NMT_Netspeed:
					FNetControlMessage<NMT_Netspeed>::Discard(Bunch);
					break;
				case NMT_Login:
					FNetControlMessage<NMT_Login>::Discard(Bunch);
					break;
				case NMT_Failure:
					FNetControlMessage<NMT_Failure>::Discard(Bunch);
					break;
				case NMT_Join:
					//FNetControlMessage<NMT_Join>::Discard(Bunch);
					break;
				case NMT_JoinSplit:
					FNetControlMessage<NMT_JoinSplit>::Discard(Bunch);
					break;
				case NMT_Skip:
					FNetControlMessage<NMT_Skip>::Discard(Bunch);
					break;
				case NMT_Abort:
					FNetControlMessage<NMT_Abort>::Discard(Bunch);
					break;
				case NMT_PCSwap:
					FNetControlMessage<NMT_PCSwap>::Discard(Bunch);
					break;
				case NMT_ActorChannelFailure:
					FNetControlMessage<NMT_ActorChannelFailure>::Discard(Bunch);
					break;
				case NMT_DebugText:
					FNetControlMessage<NMT_DebugText>::Discard(Bunch);
					break;
				case NMT_NetGUIDAssign:
					FNetControlMessage<NMT_NetGUIDAssign>::Discard(Bunch);
					break;
				case NMT_EncryptionAck:
					//FNetControlMessage<NMT_EncryptionAck>::Discard(Bunch);
					break;
				case NMT_CloseReason:
					FNetControlMessage<NMT_CloseReason>::Discard(Bunch);
					break;
				case NMT_NetPing:
					FNetControlMessage<NMT_NetPing>::Discard(Bunch);
					break;
				case NMT_BeaconWelcome:
					//FNetControlMessage<NMT_BeaconWelcome>::Discard(Bunch);
					break;
				case NMT_BeaconJoin:
					FNetControlMessage<NMT_BeaconJoin>::Discard(Bunch);
					break;
				case NMT_BeaconAssignGUID:
					FNetControlMessage<NMT_BeaconAssignGUID>::Discard(Bunch);
					break;
				case NMT_BeaconNetGUIDAck:
					FNetControlMessage<NMT_BeaconNetGUIDAck>::Discard(Bunch);
					break;
				case NMT_IrisProtocolMismatch:
					FNetControlMessage<NMT_IrisProtocolMismatch>::Discard(Bunch);
					break;
				case NMT_IrisNetRefHandleError:
					FNetControlMessage<NMT_IrisNetRefHandleError>::Discard(Bunch);
					break;
				case NMT_JoinNoPawn:
					FNetControlMessage<NMT_JoinNoPawn>::Discard(Bunch);
					break;
				case NMT_JoinNoPawnSplit:
					FNetControlMessage<NMT_JoinNoPawnSplit>::Discard(Bunch);
					break;
				case NMT_CloseChildConnection:
					FNetControlMessage<NMT_CloseChildConnection>::Discard(Bunch);
					break;
				case NMT_IrisNetRefHandleErrorWithDiagnosticMessage:
					FNetControlMessage<NMT_IrisNetRefHandleErrorWithDiagnosticMessage>::Discard(Bunch);
					break;
				case NMT_IrisProtocolMismatchWithCDOState:
					FNetControlMessage<NMT_IrisProtocolMismatchWithCDOState>::Discard(Bunch);
					break;
				default:
					// if this fails, a case is missing above for an implemented message type
					// or the connection is being sent potentially malformed packets
					// @PotentialDOSAttackDetection
					check(!FNetControlMessageInfo::IsRegistered(MessageType));

					UE_LOGF(LogNet, Log, "Received unknown control channel message %i. Closing connection.", int32(MessageType));
					Connection->Close(ENetCloseResult::ControlChannelMessageUnknown);
					return;
			}
		}
		if ( Bunch.IsError() )
		{
			UE_LOGF( LogNet, Error, "Failed to read control channel message '%ls'", FNetControlMessageInfo::GetName( MessageType ) );

			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ControlChannelMessagePayloadFail);

			bStopReadingBunch = true;
		}
	}

	if ( Bunch.IsError() )
	{
		UE_LOGF( LogNet, Error, "UControlChannel::ReceivedBunch: Failed to read control channel message" );

		if (Connection != nullptr)
		{
			Connection->Close(AddToAndConsumeChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ControlChannelMessageFail));
		}
	}
}

void UControlChannel::QueueMessage(const FOutBunch* Bunch)
{
	LLM_SCOPE_BYTAG(NetChannel);

	if (QueuedMessages.Num() >= MAX_QUEUED_CONTROL_MESSAGES)
	{
		// we're out of room in our extra buffer as well, so kill the connection
		UE_LOGF(LogNet, Log, "Overflowed control channel message queue, disconnecting client");
		// intentionally directly setting State as the messaging in Close() is not going to work in this case
		Connection->SetConnectionState(USOCK_Closed);
	}
	else
	{
		int32 Index = QueuedMessages.AddZeroed();
		FQueuedControlMessage& CurMessage = QueuedMessages[Index];

		CurMessage.Data.AddUninitialized(Bunch->GetNumBytes());
		FMemory::Memcpy(CurMessage.Data.GetData(), Bunch->GetData(), Bunch->GetNumBytes());

		CurMessage.CountBits = Bunch->GetNumBits();
	}
}

FPacketIdRange UControlChannel::SendBunch(FOutBunch* Bunch, bool Merge)
{
	// if we already have queued messages, we need to queue subsequent ones to guarantee proper ordering
	if (QueuedMessages.Num() > 0 || NumOutRec >= RELIABLE_BUFFER - 1 + Bunch->bClose)
	{
		QueueMessage(Bunch);
		return FPacketIdRange(INDEX_NONE);
	}
	else
	{
		if (!Bunch->IsError())
		{
			return Super::SendBunch(Bunch, Merge);
		}
		else
		{
			// an error here most likely indicates an unfixable error, such as the text using more than the maximum packet size
			// so there is no point in queueing it as it will just fail again
			UE_LOGF(LogNet, Error, "Control channel bunch overflowed");
			ensureMsgf(false, TEXT("Control channel bunch overflowed"));
			Connection->Close(ENetCloseResult::ControlChannelBunchOverflowed);
			return FPacketIdRange(INDEX_NONE);
		}
	}
}

void UControlChannel::Tick()
{
	Super::Tick();

	if( !OpenAcked )
	{
		int32 Count = 0;
		for (FOutBunch* Out = OutRec; Out; Out = Out->Next)
		{
			if (!Out->ReceivedAck)
			{
				Count++;
			}
		}

		if (Count > 8)
		{
			return;
		}

		// Resend any pending packets if we didn't get the appropriate acks.
		for( FOutBunch* Out=OutRec; Out; Out=Out->Next )
		{
			if( !Out->ReceivedAck )
			{
				const double Wait = Connection->Driver->GetElapsedTime() - Out->Time;
				checkSlow(Wait >= 0.0);
				if (Wait > 1.0)
				{
					UE_LOGF(LogNetTraffic, Log, "Channel %i ack timeout); resending %i...", ChIndex, Out->ChSequence );
					check(Out->bReliable);
					Connection->SendRawBunch( *Out, 0 );
				}
			}
		}
	}
	else
	{
		// attempt to send queued messages
		while (QueuedMessages.Num() > 0 && !Closing)
		{
			FControlChannelOutBunch Bunch(this, 0);
			if (Bunch.IsError())
			{
				break;
			}
			else
			{
				Bunch.bReliable = 1;
				Bunch.SerializeBits(QueuedMessages[0].Data.GetData(), QueuedMessages[0].CountBits);

				if (!Bunch.IsError())
				{
					Super::SendBunch(&Bunch, 1);
					QueuedMessages.RemoveAt(0, 1);
				}
				else
				{
					// an error here most likely indicates an unfixable error, such as the text using more than the maximum packet size
					// so there is no point in queueing it as it will just fail again
					ensureMsgf(false, TEXT("Control channel queued bunch overflowed"));
					UE_LOGF(LogNet, Error, "Control channel queued bunch overflowed");
					Connection->Close(ENetCloseResult::ControlChannelQueueBunchOverflowed);
					break;
				}
			}
		}
	}
}


FString UControlChannel::Describe()
{
	return UChannel::Describe();
}

int64 UControlChannel::SendDestructionInfo(FActorDestructionInfo* DestructionInfo)
{
	int64 NumBits = 0;

	checkf(Connection && Connection->PackageMap, TEXT("SendDestructionInfo requires a valid connection and package map: %s"), *Describe());
	checkf(DestructionInfo, TEXT("SendDestructionInfo was passed an invalid desctruction info: %s"), *Describe());

	if (!Closing && (Connection->GetConnectionState() == USOCK_Open || Connection->GetConnectionState() == USOCK_Pending))
	{
		// Outer must be valid to call PackageMap->WriteObject. In the case of streaming out levels, this can go null out of from underneath us. In that case, just skip the destruct info.
		// We assume that if server unloads a level that clients will to and this will implicitly destroy all actors in it, so not worried about leaking actors client side here.
		if (UObject* ObjOuter = DestructionInfo->ObjOuter.Get())
		{
			FOutBunch InfoBunch(Connection->PackageMap, false);
			check(!InfoBunch.IsError());
			InfoBunch.bReliable = 1;

			uint8 MessageType = NMT_DestructionInfo;
			InfoBunch << MessageType;

			EChannelCloseReason Reason = DestructionInfo->Reason;
			InfoBunch << Reason;

			Connection->PackageMap->WriteObject(InfoBunch, ObjOuter, DestructionInfo->NetGUID, DestructionInfo->PathName);

			UE_LOG(LogNetTraffic, Log, TEXT("SendDestructionInfo: NetGUID <%s> Path: %s. Bits: %" INT64_FMT), *DestructionInfo->NetGUID.ToString(), *DestructionInfo->PathName, InfoBunch.GetNumBits());
			UE_LOG(LogNetDormancy, Verbose, TEXT("SendDestructionInfo: NetGUID <%s> Path: %s. Bits: %" INT64_FMT), *DestructionInfo->NetGUID.ToString(), *DestructionInfo->PathName, InfoBunch.GetNumBits());

			SendBunch(&InfoBunch, false);

			NumBits = InfoBunch.GetNumBits();
		}
	}

	return NumBits;
}

void UControlChannel::ReceiveDestructionInfo(FInBunch& Bunch)
{
	checkf(Connection && Connection->PackageMap && Connection->Driver, TEXT("UControlChannel::ReceiveDestructionInfo requires a valid connection, package map, and driver: %s"), *Describe());
	checkf(!Connection->Driver->IsServer(), TEXT("UControlChannel::ReceiveDestructionInfo should not be called on a server: %s"), *Describe());

	EChannelCloseReason CloseReason = EChannelCloseReason::Destroyed;
	Bunch << CloseReason;

	FNetworkGUID NetGUID;
	UObject* Object = nullptr;

	if (Connection->PackageMap->SerializeObject(Bunch, UObject::StaticClass(), Object, &NetGUID))
	{
		if (AActor* TheActor = Cast<AActor>(Object))
		{
			checkf(TheActor->IsValidLowLevel(), TEXT("ReceiveDestructionInfo serialized an invalid actor: %s"), *Describe());
			checkSlow(Connection->IsValidLowLevel());
			checkSlow(Connection->Driver->IsValidLowLevel());

			if (TheActor->GetTearOff() && !Connection->Driver->ShouldClientDestroyTearOffActors())
			{
				Connection->Driver->ClientSetActorTornOff(TheActor);
			}
			else if (Dormant && (CloseReason == EChannelCloseReason::Dormancy) && !TheActor->GetTearOff())
			{
				Connection->Driver->ClientSetActorDormant(TheActor);
				Connection->Driver->NotifyActorFullyDormantForConnection(TheActor, Connection);
			}
			else if (!TheActor->bNetTemporary && TheActor->GetWorld() != nullptr && !IsEngineExitRequested() && Connection->Driver->ShouldClientDestroyActor(TheActor))
			{
				// Destroy the actor

				// Unmap any components in this actor. This will make sure that once the Actor is remapped
				// any references to components will be remapped as well.
				for (UActorComponent* Component : TheActor->GetComponents())
				{
					Connection->Driver->MoveMappedObjectToUnmapped(Component);
				}

				// Unmap this object so we can remap it if it becomes relevant again in the future
				Connection->Driver->MoveMappedObjectToUnmapped(TheActor);

				TheActor->PreDestroyFromReplication();
				TheActor->Destroy(true);

				if (UE::Net::FilterGuidRemapping != 0)
				{
					// Remove this actor's NetGUID from the list of unmapped values, it will be added back if it replicates again
					if (NetGUID.IsValid() && Connection != nullptr && Connection->Driver != nullptr && Connection->Driver->GetNetGuidCache().IsValid())
					{
						Connection->Driver->GetNetGuidCache()->ImportedNetGuids.Remove(NetGUID);
					}
				}

				if (UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap))
				{
					PackageMapClient->SetHasQueuedBunches(NetGUID, false);
				}
			}
		}
	}
}

/*-----------------------------------------------------------------------------
	UActorChannel.
-----------------------------------------------------------------------------*/

void UActorChannel::Init( UNetConnection* InConnection, int32 InChannelIndex, EChannelCreateFlags CreateFlags )
{
	Super::Init( InConnection, InChannelIndex, CreateFlags );

	RelevantTime			= Connection->Driver->GetElapsedTime();
	LastUpdateTime			= Connection->Driver->GetElapsedTime() - Connection->Driver->SpawnPrioritySeconds;
	bForceCompareProperties	= false;
	bActorIsPendingKill		= false;
	CustomTimeDilation		= 1.0f;

	UE_LOGF(LogNetLifecycle, Log, "UActorChannel::Init: ChIndex: %d Actor: %ls", ChIndex, *GetFullNameSafe(Actor));
}

void UActorChannel::SetClosingFlag()
{
	if( Actor && Connection )
	{
		Connection->RemoveActorChannel( Actor );
	}

	UChannel::SetClosingFlag();
}

void UActorChannel::ReleaseReferences(bool bKeepReplicators)
{
	CleanupReplicators(bKeepReplicators);
	GetCreatedSubObjects().Empty();
	Actor = nullptr;
}

void UActorChannel::BreakAndReleaseReferences()
{
	Broken = true;
	ReleaseReferences(false);
}

int64 UActorChannel::Close(EChannelCloseReason Reason)
{
	UE_LOGF(LogNetLifecycle, Log, "UActorChannel::Close: ChIndex: %d Actor: %ls", ChIndex, *GetFullNameSafe(Actor));

	FScopedRepContext RepContext(Connection, Actor);

	UE_LOGF(LogNetTraffic, Log, "UActorChannel::Close: ChIndex: %d, Actor: %ls, Reason: %ls", ChIndex, *GetFullNameSafe(Actor), LexToString(Reason));
	int64 NumBits = UChannel::Close(Reason);

	if (Actor != nullptr)
	{
		bool bKeepReplicators = false;		// If we keep replicators around, we can use them to determine if the actor changed since it went dormant

		if (Connection)
		{
			if (Reason == EChannelCloseReason::Dormancy)
			{
				const bool bIsDriverValid = Connection->Driver != nullptr;
				const bool bIsServer = bIsDriverValid && Connection->Driver->IsServer();
				if (bIsDriverValid)
				{
					if (!bIsServer)
					{
						Connection->Driver->ClientSetActorDormant(Actor);
					}
					else
					{
						ensureMsgf(Actor->NetDormancy > DORM_Awake, TEXT("Dormancy should have been canceled if game code changed NetDormancy: %s [%s]"), *GetFullNameSafe(Actor), *UEnum::GetValueAsString<ENetDormancy>(Actor->NetDormancy));
					}

					Connection->Driver->NotifyActorFullyDormantForConnection(Actor, Connection);
				}

				// Validation checking
				// We need to keep the replicators around so we can reuse them.
				bKeepReplicators = (GNetDormancyValidate > 0) || (bIsServer && GbNetReuseReplicatorsForDormantObjects);
			}

			// SetClosingFlag() might have already done this, but we need to make sure as that won't get called if the connection itself has already been closed
			Connection->RemoveActorChannel( Actor );
		}

		ReleaseReferences(bKeepReplicators);
	}

	return NumBits;
}

void UActorChannel::CleanupReplicators(const bool bKeepReplicators)
{
#if UE_REPLICATED_OBJECT_REFCOUNTING
	TArray<TWeakObjectPtr<UObject>, TInlineAllocator<16>> ReferencesToRemove;
#endif

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
    // The reference swap is only needed when we keep track of individual references since the refcount stays identical.
    // So we only keep track of swapped objects when we run with the checks enabled.
	TArray<TWeakObjectPtr<UObject>, TInlineAllocator<16>> ReferencesToSwap;
#endif

	// Cleanup or save replicators
	for (auto CompIt = ReplicationMap.CreateIterator(); CompIt; ++CompIt)
	{
		TSharedRef< FObjectReplicator >& ObjectReplicatorRef = CompIt.Value();

		// NOTE: FObjectReplicator::GetObject is just going to return a raw Object Pointer,
		// so it won't actually check to see whether or not the Object was marked PendingKill/Garbage.
		if (bKeepReplicators && ObjectReplicatorRef->GetObject() != nullptr)
		{
			LLM_SCOPE_BYTAG(NetConnection);

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
			// Ignore the main actor's replicator
			if (Actor != ObjectReplicatorRef->ObjectPtr)
			{
				ReferencesToSwap.Add(ObjectReplicatorRef->GetWeakObjectPtr());
			}
#endif

			// If we want to keep the replication state of the actor/sub-objects around, transfer ownership to the connection
			// This way, if this actor opens another channel on this connection, we can reclaim or use this replicator to compare state, etc.
			// For example, we may want to see if any state changed since the actor went dormant, and is now active again.
			//	NOTE - Commenting out this assert, since the case that it's happening for should be benign.
			//	Here is what is likely happening:
			//		We move a channel to the KeepProcessingActorChannelBunchesMap
			//		While the channel is on this list, we also re-open a new channel using the same actor
			//		KeepProcessingActorChannelBunchesMap will get in here, then when the channel closes a second time, we'll hit this assert
			//		It should be okay to just set the most recent replicator
			Connection->StoreDormantReplicator(Actor, ObjectReplicatorRef->GetObject(), ObjectReplicatorRef);
			ObjectReplicatorRef->StopReplicating(this);		// Stop replicating on this channel
		}
		else
		{
#if UE_REPLICATED_OBJECT_REFCOUNTING
			// Ignore the main actor's replicator
			if (Actor != ObjectReplicatorRef->ObjectPtr)
			{
				ReferencesToRemove.Add(ObjectReplicatorRef->GetWeakObjectPtr());
			}
#endif
			ObjectReplicatorRef->CleanUp();
		}
	}

#if UE_REPLICATED_OBJECT_REFCOUNTING
	if (ReferencesToRemove.Num() > 0)
	{
		Connection->Driver->GetNetworkObjectList().RemoveMultipleSubObjectChannelReference(Actor, ReferencesToRemove, this);
	}
#endif

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
	if (ReferencesToSwap.Num() > 0)
	{
		Connection->Driver->GetNetworkObjectList().SwapMultipleReferencesForDormancy(Actor, ReferencesToSwap, this, Connection);
	}
#endif

	ReplicationMap.Empty();

	ActorReplicator.Reset();
}

void UActorChannel::MoveMappedObjectToUnmapped(const UObject* Object)
{
	if (Connection && Connection->Driver)
	{
		Connection->Driver->MoveMappedObjectToUnmapped(Object);
	}
}

void UActorChannel::DestroyActorAndComponents(EChannelCloseReason Reason)
{
	// Destroy any sub-objects we created
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (UObject* SubObject : CreateSubObjects)
	{
		if (SubObject != nullptr)
		{
			// Unmap this object so we can remap it if it becomes relevant again in the future
			MoveMappedObjectToUnmapped(SubObject);

			if (Connection != nullptr && Connection->Driver != nullptr)
			{
				Connection->Driver->NotifySubObjectDestroyed(SubObject);
			}

			Actor->OnSubobjectDestroyFromReplication(SubObject);

			SubObject->PreDestroyFromReplication();
			SubObject->MarkAsGarbage();
		}
	}

	CreateSubObjects.Empty();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Destroy the actor
	if (Actor != nullptr)
	{
		// Unmap any components in this actor. This will make sure that once the Actor is remapped
		// any references to components will be remapped as well.
		for (UActorComponent* Component : Actor->GetComponents())
		{
			MoveMappedObjectToUnmapped(Component);
		}

		// Also unmap any stably-named subobjects we didn't create
		if (UE::Net::Private::bRemapStableSubobjects)
		{
			TArray<UObject*> Inners;
			GetObjectsWithOuter(Actor, Inners);
			for (const UObject* Inner : Inners)
			{
				if (Inner->IsNameStableForNetworking())
				{
					MoveMappedObjectToUnmapped(Inner);
				}
			}
		}

		// Unmap this object so we can remap it if it becomes relevant again in the future
		MoveMappedObjectToUnmapped(Actor);

		Actor->PreDestroyFromReplication();
		Actor->Destroy(true);
	}

	if (UE::Net::FilterGuidRemapping != 0)
	{
		// Remove this actor's NetGUID from the list of unmapped values, it will be added back if it replicates again
		if (ActorNetGUID.IsValid() && Connection != nullptr && Connection->Driver != nullptr && Connection->Driver->GetNetGuidCache().IsValid())
		{
			Connection->Driver->GetNetGuidCache()->ImportedNetGuids.Remove(ActorNetGUID);
		}
	}
}

bool UActorChannel::CleanUp(const bool bForDestroy, EChannelCloseReason CloseReason)
{
	UE_LOGF(LogNetLifecycle, Log, "UActorChannel::CleanUp: ChIndex: %d Actor: %ls", ChIndex, *GetFullNameSafe(Actor));

	SCOPE_CYCLE_COUNTER(Stat_ActorChanCleanUp);

	checkf(Connection != nullptr, TEXT("UActorChannel::CleanUp: Connection is null!"));
	checkf(Connection->Driver != nullptr, TEXT("UActorChannel::CleanUp: Connection->Driver is null!"));

	Connection->Driver->NotifyActorChannelCleanedUp(this, CloseReason);

	const bool bIsServer = Connection->Driver->IsServer();

	UE_LOGF( LogNetTraffic, Log, "UActorChannel::CleanUp (ChIndex: %i): %ls | NetGuid: %ls | Reason: %ls", ChIndex, *GetFullNameSafe(Actor), *ActorNetGUID.ToString(), LexToString(CloseReason));

	if (!bIsServer && QueuedBunches.Num() > 0 && ChIndex >= 0 && !bForDestroy)
	{
		LLM_SCOPE_BYTAG(NetConnection);

		// Allow channels in replays to be cleaned up even if the guid was never set
		checkf(ActorNetGUID.IsValid() || Connection->IsInternalAck(), TEXT("UActorChannel::Cleanup: ActorNetGUID is invalid! Channel: %i"), ChIndex);

		if (ActorNetGUID.IsValid())
		{
			auto& ChannelsStillProcessing = Connection->KeepProcessingActorChannelBunchesMap.FindOrAdd(ActorNetGUID);

#if DO_CHECK
			if (ensureMsgf(!ChannelsStillProcessing.Contains(this), TEXT("UActorChannel::CleanUp encountered a channel already within the KeepProcessingActorChannelBunchMap. Channel: %i"), ChIndex))
#endif // #if DO_CHECK
			{
				UE_LOGF(LogNet, VeryVerbose, "UActorChannel::CleanUp: Adding to KeepProcessingActorChannelBunchesMap. Channel: %i, Num: %i", ChIndex, Connection->KeepProcessingActorChannelBunchesMap.Num());

				// Remember the connection, since CleanUp below will NULL it
				UNetConnection* OldConnection = Connection;

				// This will unregister the channel, and make it free for opening again
				// We need to do this, since the server will assume this channel is free once we ack this packet
				Super::CleanUp(bForDestroy, CloseReason);

				// Restore connection property since we'll need it for processing bunches (the Super::CleanUp call above NULL'd it)
				Connection = OldConnection;

				QueuedCloseReason = CloseReason;

				// Add this channel to the KeepProcessingActorChannelBunchesMap list
				ChannelsStillProcessing.Add(ObjectPtrWrap(this));

				// We set ChIndex to -1 to signify that we've already been "closed" but we aren't done processing bunches
				ChIndex = -1;

				// Return false so we won't do pending kill yet
				return false;
			}
		}
	}

	bool bWasDormant = false;

	// If we're the client, destroy this actor.
	if (!bIsServer)
	{
		check(Actor == NULL || Actor->IsValidLowLevel());
		checkSlow(Connection->IsValidLowLevel());
		checkSlow(Connection->Driver->IsValidLowLevel());

		// Allow actors to customize behavior on relevancy changes.
		bool bHandled = false;
		if (IReplicatedObjectInterface* ReplicatedObject = Cast<IReplicatedObjectInterface>(Actor))
		{
			if (CloseReason == EChannelCloseReason::Relevancy)
			{
				ReplicatedObject->OnLostRelevancy();
				if (!ReplicatedObject->ShouldClientDestroyWhenNotRelevant())
				{
					UE_LOGF(LogNetTraffic, VeryVerbose, "UActorChannel::CleanUp: Closed for ReplicatedObject Relevancy. %ls - [%ls] FullNetGUIDPath: %ls", *Describe(), *ActorNetGUID.ToString(), *Connection->Driver->GetNetGuidCache()->FullNetGUIDPath(ActorNetGUID));
					bHandled = true;
				}
			}
		}

		if (!bHandled && Actor != nullptr)
		{
			if (CloseReason == EChannelCloseReason::TearOff && !Actor->GetTearOff())
			{
				UE_LOGF(LogNetTraffic, Verbose, "UActorChannel::CleanUp: Closed for TearOff but actor flag not set, last property update may have been dropped: %ls", *Describe());
				UE::Net::FTearOffSetter::SetTearOff(Actor);
			}

			if (Actor->GetTearOff() && !Connection->Driver->ShouldClientDestroyTearOffActors())
			{
				if (!bTornOff)
				{
					bTornOff = true;
					Connection->Driver->ClientSetActorTornOff(Actor);
				}
			}
			else if (Dormant && (CloseReason == EChannelCloseReason::Dormancy) && !Actor->GetTearOff())
			{
				Connection->Driver->ClientSetActorDormant(Actor);
				Connection->Driver->NotifyActorFullyDormantForConnection(Actor, Connection);
				bWasDormant = true;
			}
			else if (UE::Net::bSkipDestroyNetStartupActorsOnChannelCloseDueToLevelUnloaded && (CloseReason == EChannelCloseReason::LevelUnloaded) && Actor->IsNetStartupActor())
			{
				UE_LOGF(LogNetTraffic, Verbose, "UActorChannel::CleanUp: Skipped Destroying NetStartupActor %ls due to EChannelCloseReason::LevelUnloaded", *Describe());
			}
			else if (!Actor->bNetTemporary && Actor->GetWorld() != NULL && !IsEngineExitRequested() && Connection->Driver->ShouldClientDestroyActor(Actor, CloseReason))
			{
				UE_LOGF(LogNetDormancy, Verbose, "UActorChannel::CleanUp: Destroying Actor (ChIndex: %i): %ls", ChIndex, *GetFullNameSafe(Actor));

				DestroyActorAndComponents(CloseReason);
			}
		}
	}

	// Remove from hash and stuff.
	SetClosingFlag();

	// If this actor is going dormant (and we are a client), keep the replicators around, we need them to run the business logic for updating unmapped properties
	const bool bKeepReplicators = !bForDestroy && bWasDormant && (!bIsServer || GbNetReuseReplicatorsForDormantObjects);

	CleanupReplicators( bKeepReplicators );

	// We don't care about any leftover pending guids at this point
	PendingGuidResolves.Empty();
	QueuedBunchObjectReferences.Empty();

	// Free export bunches list
	for (FOutBunch* QueuedOutBunch : QueuedExportBunches)
	{
		delete QueuedOutBunch;
	}

	QueuedExportBunches.Empty();

	// Free the must be mapped list
	QueuedMustBeMappedGuidsInLastBunch.Empty();

	if (QueuedBunches.Num() > 0)
	{
		// Free any queued bunches
		for (FInBunch* QueuedInBunch : QueuedBunches)
		{
			delete QueuedInBunch;
		}

		QueuedBunches.Empty();

		if (UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap))
		{
			PackageMapClient->SetHasQueuedBunches(ActorNetGUID, false);
		}
	}

	// We check for -1 here, which will be true if this channel has already been closed but still needed to process bunches before fully closing
	if (ChIndex >= 0)
	{
		return Super::CleanUp(bForDestroy, CloseReason);
	}
	else
	{
		// Because we set Connection = OldConnection; above when we set ChIndex to -1, we have to null it here explicitly to make sure the connection is cleared by the time we leave CleanUp
		Connection = nullptr;
	}

	return true;
}

void UActorChannel::ReceivedNak( int32 NakPacketId )
{
	UChannel::ReceivedNak(NakPacketId);
	for (auto CompIt = ReplicationMap.CreateIterator(); CompIt; ++CompIt)
	{
		CompIt.Value()->ReceivedNak(NakPacketId);
	}

#if NET_ENABLE_SUBOBJECT_REPKEYS
	// Reset any subobject RepKeys that were sent on this packetId
	FPacketRepKeyInfo * Info = SubobjectNakMap.Find(NakPacketId % SubobjectRepKeyBufferSize);
	if (Info)
	{
		if (Info->PacketID == NakPacketId)
		{
			UE_LOGF(LogNetTraffic, Verbose, "ActorChannel[%d]: Reseting object keys due to Nak: %d", ChIndex, NakPacketId);
			for (auto It = Info->ObjKeys.CreateIterator(); It; ++It)
			{
				SubobjectRepKeyMap.FindOrAdd(*It) = INDEX_NONE;
				UE_LOGF(LogNetTraffic, Verbose, "    %d", *It);
			}
		}
	}
#endif // NET_ENABLE_SUBOBJECT_REPKEYS
}

void UActorChannel::SetChannelActor(AActor* InActor, ESetChannelActorFlags Flags)
{
	check(!Closing);
	check(Actor == nullptr);

	// Sanity check that the actor is in the same level collection as the channel's driver.
	const UWorld* const World = Connection->Driver ? Connection->Driver->GetWorld() : nullptr;
	if (World && InActor)
	{
		const ULevel* const CachedLevel = InActor->GetLevel();
		const FLevelCollection* const ActorCollection = CachedLevel ? CachedLevel->GetCachedLevelCollection() : nullptr;
		if (ActorCollection &&
			ActorCollection->GetNetDriver() != Connection->Driver &&
			ActorCollection->GetDemoNetDriver() != Connection->Driver)
		{
			UE_LOGF(LogNet, Verbose, "UActorChannel::SetChannelActor: actor %ls is not in the same level collection as the net driver (%ls)!", *GetFullNameSafe(InActor), *GetFullNameSafe(Connection->Driver));
		}
	}

	// Set stuff.
	Actor = InActor;

	// We could check Actor->IsPendingKill here, but that would suppress the warning later.
	// Further, expect calling code to do these checks.
	bActorIsPendingKill = false;

	UE_LOGF(LogNetTraffic, VeryVerbose, "SetChannelActor (ChIndex: %i): Actor: %ls | NetGUID: %ls", ChIndex, *GetFullNameSafe(Actor), *ActorNetGUID.ToString() );

	// Internal ack connections never add to PendingOutRec
	if (Connection->PendingOutRec.IsValidIndex(ChIndex) && Connection->PendingOutRec[ChIndex] > 0)
	{
		// send empty reliable bunches to synchronize both sides
		// UE_LOGF(LogNetTraffic, Log, "%i Actor %ls WILL BE sending %i vs first %i", ChIndex, *Actor->GetName(), Connection->PendingOutRec[ChIndex],Connection->OutReliable[ChIndex]);
		int32 RealOutReliable = Connection->OutReliable[ChIndex];
		Connection->OutReliable[ChIndex] = Connection->PendingOutRec[ChIndex] - 1;
		while (Connection->PendingOutRec[ChIndex] <= RealOutReliable)
		{
			// UE_LOGF(LogNetTraffic, Log, "%i SYNCHRONIZING by sending %i", ChIndex, Connection->PendingOutRec[ChIndex]);

			FOutBunch Bunch(this, 0);

			if (!Bunch.IsError())
			{
				Bunch.bReliable = true;
				SendBunch(&Bunch, 0);
				Connection->PendingOutRec[ChIndex]++;
			}
			else
			{
				// While loop will be infinite without either fatal or break.
				UE_LOGF(LogNetTraffic, Fatal, "SetChannelActor failed. Overflow while sending reliable bunch synchronization.");
				break;
			}
		}

		Connection->OutReliable[ChIndex] = RealOutReliable;
		Connection->PendingOutRec[ChIndex] = 0;
	}

	if (Actor)
	{
		// Add to map.
		Connection->AddActorChannel(Actor, this);

		check(!ReplicationMap.Contains(Actor));

		// Create the actor replicator, and store a quick access pointer to it
		if (!EnumHasAnyFlags(Flags, ESetChannelActorFlags::SkipReplicatorCreation))
		{
			ActorReplicator = FindOrCreateReplicator(Actor);
		}

		if (!EnumHasAnyFlags(Flags, ESetChannelActorFlags::SkipMarkActive))
		{
			// Remove from connection's dormancy lists
			Connection->Driver->GetNetworkObjectList().MarkActive(Actor, Connection, Connection->Driver);
			Connection->Driver->GetNetworkObjectList().ClearRecentlyDormantConnection(Actor, Connection, Connection->Driver);
		}
	}
}

void UActorChannel::NotifyActorChannelOpen(AActor* InActor, FInBunch& InBunch)
{
	UNetDriver* const NetDriver = (Connection && Connection->Driver) ? Connection->Driver : nullptr;
	UWorld* const World = (NetDriver && NetDriver->World) ? NetDriver->World.Get() : nullptr;

	FWorldContext* const Context = GEngine->GetWorldContextFromWorld(World);
	if (Context != nullptr)
	{
		for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver != nullptr)
			{
				Driver.NetDriver->NotifyActorChannelOpen(this, InActor);
			}
		}
	}

	Actor->OnActorChannelOpen(InBunch, Connection);

	// Do not update client dormancy or other net drivers if this is a destruction info, those should be handled already in UNetDriver::NotifyActorDestroyed
	if (NetDriver && !NetDriver->IsServer() && !InBunch.bClose)
	{
		if (Actor->NetDormancy > DORM_Awake)
		{
			ENetDormancy OldDormancy = Actor->NetDormancy;

			Actor->NetDormancy = DORM_Awake;

			if (Context != nullptr)
			{
				for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
				{
					if (Driver.NetDriver != nullptr && Driver.NetDriver != NetDriver && Driver.NetDriver->ShouldReplicateActor(InActor))
					{
						Driver.NetDriver->NotifyActorClientDormancyChanged(InActor, OldDormancy);
					}
				}
			}
		}
	}
}

void UActorChannel::Tick()
{
	Super::Tick();
	ProcessQueuedBunches();
}

bool UActorChannel::CanStopTicking() const
{
	return Super::CanStopTicking() && PendingGuidResolves.Num() == 0 && QueuedBunches.Num() == 0;
}

bool UActorChannel::ShouldProcessAllQueuedBunches(float CurrentTimeSeconds)
{
	using namespace UE::Net;

	if (Connection->Driver->GetIncomingBunchFrameProcessingTimeLimit() <= 0.0f)
	{
		return true;
	}

	const float SecondsWithQueuedBunches = static_cast<float>(CurrentTimeSeconds - QueuedBunchStartTime);
	const bool bExceededFailsafeTime = SecondsWithQueuedBunches > QueuedBunchTimeFailsafeSeconds;

	if (bExceededFailsafeTime)
	{
		Connection->Driver->AddQueuedBunchFailsafeChannel();
		UE_LOGF(LogNet, Verbose, "UActorChannel::ProcessQueuedBunches hit frame time failsafe after %.3f seconds. Processing entire queue of %d bunches for channel: %ls", SecondsWithQueuedBunches, QueuedBunches.Num(), *Describe());
	}

	return bExceededFailsafeTime;
};

bool UActorChannel::ProcessQueuedBunches()
{
	if (PendingGuidResolves.Num() == 0 && QueuedBunches.Num() == 0)
	{
		return true;
	}

	const uint64 QueueBunchStartCycles = FPlatformTime::Cycles64();

	// Try to resolve any guids that are holding up the network stream on this channel
	// TODO: This could take a non-trivial amount of time since both GetObjectFromNetGUID
	// and IsGUIDBroken may do Map Lookups and GetObjectFromNetGUID will attempt
	// to resolve weak objects.
	FNetGUIDCache* GuidCache = Connection->Driver->GetNetGuidCache().Get();
	for (auto It = PendingGuidResolves.CreateIterator(); It; ++It)
	{
		if (GuidCache->GetObjectFromNetGUID(*It, true))
		{
			// This guid is now resolved, we can remove it from the pending guid list
			It.RemoveCurrent();
		}
		else if (GuidCache->IsGUIDBroken(*It, true))
		{
			// This guid is broken, remove it, and warn
			UE_LOGF(LogNet, Warning, "UActorChannel::ProcessQueuedBunches: Guid is broken. NetGUID: %ls, ChIndex: %i, Actor: %ls", *It->ToString(), ChIndex, *GetPathNameSafe(Actor));
			It.RemoveCurrent();
		}
	}

	if (QueuedBunches.Num() == 0)
	{
		return true;
	}

	// Always update this when there are bunches remaining, because we may not hitch on the frame that triggers a warning.
	bSuppressQueuedBunchWarningsDueToHitches |= Connection->Driver->DidHitchLastFrame();

	// Instant replays are played back in a duplicated level collection, so if this is instant replay
	// playback, the driver's DuplicateLevelID will be something other than INDEX_NONE.
	const int BunchTimeLimit = Connection->Driver->GetDuplicateLevelID() == INDEX_NONE ?
		CVarNetProcessQueuedBunchesMillisecondLimit.GetValueOnGameThread() :
		CVarNetInstantReplayProcessQueuedBunchesMillisecondLimit.GetValueOnGameThread();

	const bool bHasTimeToProcess = BunchTimeLimit == 0 || Connection->Driver->ProcessQueuedBunchesCurrentFrameMilliseconds < BunchTimeLimit;

	// If we don't have any time, then don't bother doing anything (including warning) as that may make things worse.
	if (bHasTimeToProcess)
	{
		// We can process at least some of the queued up bunches if ALL of these are true:
		//	1. We no longer have any pending guids to load
		//	2. We aren't still processing bunches on another channel that this actor was previously on
		//	3. We haven't spent too much time yet this frame processing queued bunches
		//	4. The driver isn't requesting queuing for this GUID
		if (PendingGuidResolves.Num() == 0
			&& (ChIndex == -1 || !Connection->KeepProcessingActorChannelBunchesMap.Contains(ActorNetGUID))
			&& !Connection->Driver->ShouldQueueBunchesForActorGUID(ActorNetGUID))
		{
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ProcessQueuedBunches time"), STAT_ProcessQueuedBunchesTime, STATGROUP_Net);

			// Check failsafe timer so we don't end up starving the queue indefinitely due to frame time limits
			const bool bProcessAllBunches = ShouldProcessAllQueuedBunches(FPlatformTime::ToSeconds64(QueueBunchStartCycles));

			int32 BunchIndex = 0;
			while ((BunchIndex < QueuedBunches.Num()) && (bProcessAllBunches || !Connection->Driver->HasExceededIncomingBunchFrameProcessingTime()))
			{
				FInBunch* QueuedInBunch = QueuedBunches[BunchIndex];

				ProcessBunch(*QueuedInBunch);
				delete QueuedInBunch;
				QueuedBunches[BunchIndex] = nullptr;

				++BunchIndex;
			}

			UE_LOGF(LogNet, VeryVerbose, "UActorChannel::ProcessQueuedBunches: Flushing %i of %i queued bunches. ChIndex: %i, Actor: %ls", BunchIndex, QueuedBunches.Num(), ChIndex, Actor != NULL ? *Actor->GetPathName() : TEXT("NULL"));

			// Remove processed bunches from the queue
			QueuedBunches.RemoveAt(0, BunchIndex);

			// Call any onreps that were delayed because we were queuing bunches
			for (auto& ReplicatorPair : ReplicationMap)
			{
				ReplicatorPair.Value->CallRepNotifies(true);
			}

			if (UPackageMapClient * PackageMapClient = Cast< UPackageMapClient >(Connection->PackageMap))
			{
#if CSV_PROFILER_STATS
				FNetGUIDCache::FIsOwnerOrPawnHelper Helper(GuidCache, Connection->OwningActor, Actor);
#endif

				PackageMapClient->SetHasQueuedBunches(ActorNetGUID, !QueuedBunches.IsEmpty());
			}

			// We don't know exactly which bunches were referring to which objects, so we have to conservatively keep them all around.
			if (QueuedBunches.IsEmpty())
			{
				QueuedBunchObjectReferences.Empty();
			}
		}
		else
		{
			if ((FPlatformTime::Seconds() - QueuedBunchStartTime) > UE::Net::QueuedBunchTimeoutSeconds)
			{
				if (!bSuppressQueuedBunchWarningsDueToHitches && FPlatformProperties::RequiresCookedData())
				{
					UE_LOGF(LogNet, Warning, "UActorChannel::ProcessQueuedBunches: Queued bunches for longer than normal. ChIndex: %i, Actor: %ls, Queued: %i, PendingGuidResolves: %i", ChIndex, *GetPathNameSafe(Actor), QueuedBunches.Num(), PendingGuidResolves.Num());

					if (UE_LOG_ACTIVE(LogNet, Log))
					{
						for (const FNetworkGUID& Guid : PendingGuidResolves)
						{
							UE_LOGF(LogNet, Log, "  PendingGuidResolve [%ls] %ls", *Guid.ToString(), *GuidCache->Describe(Guid));

							const FNetworkGUID OuterGuid = GuidCache->GetOuterNetGUID(Guid);
							UE_LOGF(LogNet, Log, "      OuterGuid [%ls] %ls", *OuterGuid.ToString(), *GuidCache->Describe(OuterGuid));
						}
					}
				}

				QueuedBunchStartTime = FPlatformTime::Seconds();
			}
		}

		// Update the driver with our time spent
		const uint64 QueueBunchEndCycles = FPlatformTime::Cycles64();
		const uint64 QueueBunchDeltaCycles = QueueBunchEndCycles - QueueBunchStartCycles;
		const float QueueBunchDeltaMilliseconds = static_cast<float>(FPlatformTime::ToMilliseconds64(QueueBunchDeltaCycles));

		Connection->Driver->ProcessQueuedBunchesCurrentFrameMilliseconds += QueueBunchDeltaMilliseconds;
	}

	// Return true if we are done processing queued bunches
	return QueuedBunches.Num() == 0;
}

void UActorChannel::ReceivedBunch( FInBunch & Bunch )
{
	LLM_SCOPE_BYTAG(NetChannel);

	check(!Closing);

	if (Broken || bTornOff)
	{
		return;
	}

	TArray<TPair<FNetworkGUID, UObject*>> QueuedObjectsToTrack;

	if (Connection->Driver->IsServer())
	{
		if (Bunch.bHasMustBeMappedGUIDs)
		{
			UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReceivedBunch: Client attempted to set bHasMustBeMappedGUIDs. Actor: %ls", *GetNameSafe(Actor));

			Bunch.SetError();
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ClientHasMustBeMappedGUIDs);

			return;
		}
	}
	else
	{
		if (Bunch.bHasMustBeMappedGUIDs)
		{
			UE_NET_TRACE_SCOPE(MustBeMappedGUIDs, Bunch, Connection->GetInTraceCollector(), ENetTraceVerbosity::Trace);

			// If this bunch has any guids that must be mapped, we need to wait until they resolve before we can
			// process the rest of the stream on this channel
			uint16 NumMustBeMappedGUIDs = 0;
			Bunch << NumMustBeMappedGUIDs;

			QueuedObjectsToTrack.Reserve(NumMustBeMappedGUIDs);
			//UE_LOGF( LogNetTraffic, Warning, "Read must be mapped GUID's. NumMustBeMappedGUIDs: %i", NumMustBeMappedGUIDs );

			FNetGUIDCache* GuidCache = Connection->Driver->GetNetGuidCache().Get();

#if CSV_PROFILER_STATS
			FNetGUIDCache::FIsOwnerOrPawnHelper Helper(GuidCache, Connection->OwningActor, Actor);
#endif

			for (int32 i = 0; i < NumMustBeMappedGUIDs; i++)
			{
				FNetworkGUID NetGUID;
				Bunch << NetGUID;

				// If we have async package map loading disabled, we have to ignore NumMustBeMappedGUIDs
				//	(this is due to the fact that async loading could have been enabled on the server side)
				if (!GuidCache->ShouldAsyncLoad())
				{
					continue;
				}

				if (FNetGuidCacheObject const * const GuidCacheObject = GuidCache->GetCacheObject(NetGUID))
				{
					if (UObject* const Object = GuidCacheObject->Object.Get())
					{
						// Note this must be mapped guid / object pair.
						// If we are already queuing bunches, then we'll track it below.
						QueuedObjectsToTrack.Emplace(NetGUID, Object);
					}
					else
					{
						PendingGuidResolves.Add(NetGUID);

						// Start ticking this channel so that we try to resolve the pending GUID
						Connection->StartTickingChannel(this);

						// We know we're going to be queuing bunches and will need to track this object,
						// so don't bother throwing it in the array, and just track it immediately.
						QueuedBunchObjectReferences.Add(GuidCache->TrackQueuedBunchObjectReference(NetGUID, nullptr));
					}
				}
				else
				{
					// This GUID better have been exported before we get here, which means it must be registered by now
					UE_LOGF(LogNet, Warning, "UActorChannel::ReceivedBunch: Received a MustBeMappedGUID that is not registered. ChIndex: %i NetGUID: %ls Channel: %ls Bunch: %ls", ChIndex, *NetGUID.ToString(), *Describe(), *Bunch.ToString());

					Bunch.SetError();
					AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::UnregisteredMustBeMappedGUID);

					return;
				}
			}
		}

		if (Actor == NULL && Bunch.bOpen)
		{
			// Take a sneak peak at the actor guid so we have a copy of it now
			FBitReaderMark Mark(Bunch);

			NET_CHECKSUM(Bunch);

			Bunch << ActorNetGUID;

			Mark.Pop(Bunch);

			// we can now map guid to channel, even if all the bunches get queued
			if (Connection->IsInternalAck())
			{
				Connection->NotifyActorNetGUID(this);
			}
		}

		// We need to queue this bunch if any of these are true:
		//	1. We have pending guids to resolve
		//	2. We already have queued up bunches
		//	3. If this actor was previously on a channel that is now still processing bunches after a close
		//	4. The driver is requesting queuing for this GUID
		//	5. We are time-boxing incoming bunch processing and have run out of time this frame
		if (PendingGuidResolves.Num() > 0 || QueuedBunches.Num() > 0 || Connection->KeepProcessingActorChannelBunchesMap.Contains(ActorNetGUID) ||
			Connection->Driver->ShouldQueueBunchesForActorGUID(ActorNetGUID) ||
			Connection->Driver->HasExceededIncomingBunchFrameProcessingTime())
		{
			if (Connection->KeepProcessingActorChannelBunchesMap.Contains(ActorNetGUID))
			{
				UE_LOGF(LogNet, Verbose, "UActorChannel::ReceivedBunch: Queuing bunch because another channel (that closed) is processing bunches for this guid still. ActorNetGUID: %ls", *ActorNetGUID.ToString());
			}

			if (QueuedBunches.Num() == 0)
			{
				// Remember when we first started queuing
				QueuedBunchStartTime = FPlatformTime::Seconds();
				bSuppressQueuedBunchWarningsDueToHitches = false;
			}

			QueuedBunches.Add(new FInBunch(Bunch));

			// Start ticking this channel so we can process the queued bunches when possible
			Connection->StartTickingChannel(this);

			// Register this as being queued
			UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap);

			if (PackageMapClient)
			{
				PackageMapClient->SetHasQueuedBunches(ActorNetGUID, true);
			}

			if (FNetGUIDCache* LocalGuidCache = Connection->Driver->GetNetGuidCache().Get())
			{
				for (TPair<FNetworkGUID, UObject*>& NetGUIDObjectPair : QueuedObjectsToTrack)
				{
					QueuedBunchObjectReferences.Add(LocalGuidCache->TrackQueuedBunchObjectReference(NetGUIDObjectPair.Key, NetGUIDObjectPair.Value));
				}
			}

			return;
		}
	}

	// We can process this bunch now
	ProcessBunch(Bunch);
}

void UActorChannel::ProcessBunchInternal(FInBunch & Bunch)
{
	if ( Broken )
	{
		return;
	}

	uint64 StartTimeCycles = FPlatformTime::Cycles64();
	ON_SCOPE_EXIT
	{
		if (Connection->Driver->GetIncomingBunchFrameProcessingTimeLimit() > 0.0f)
		{
			const uint64 DeltaCycles = FPlatformTime::Cycles64() - StartTimeCycles;
			const double DeltaMS = FPlatformTime::ToMilliseconds64(DeltaCycles);
			Connection->Driver->AddBunchProcessingFrameTimeMS(static_cast<float>(DeltaMS));
		}
	};

	FReplicationFlags RepFlags;

	// ------------------------------------------------------------
	// Initialize client if first time through.
	// ------------------------------------------------------------
	bool bSpawnedNewActor = false;	// If this turns to true, we know an actor was spawned (rather than found)
	if( Actor == NULL )
	{
		if( !Bunch.bOpen )
		{
			// This absolutely shouldn't happen anymore, since we no longer process packets until channel is fully open early on
			UE_LOGF(LogNetTraffic, Error, "UActorChannel::ProcessBunch: New actor channel received non-open packet. bOpen: %i, bClose: %i, bReliable: %i, bPartial: %i, bPartialInitial: %i, bPartialFinal: %i, ChName: %ls, ChIndex: %i, Closing: %i, OpenedLocally: %i, OpenAcked: %i, NetGUID: %ls", (int)Bunch.bOpen, (int)Bunch.bClose, (int)Bunch.bReliable, (int)Bunch.bPartial, (int)Bunch.bPartialInitial, (int)Bunch.bPartialFinal, *ChName.ToString(), ChIndex, (int)Closing, (int)OpenedLocally, (int)OpenAcked, *ActorNetGUID.ToString() );
			return;
		}

		UE_NET_TRACE_SCOPE(NewActor, Bunch, Connection->GetInTraceCollector(), ENetTraceVerbosity::Trace);

		AActor* NewChannelActor = NULL;
		bSpawnedNewActor = Connection->PackageMap->SerializeNewActor(Bunch, this, NewChannelActor);

		// We are unsynchronized. Instead of crashing, let's try to recover.
		if (!IsValid(NewChannelActor))
		{
			// got a redundant destruction info, possible when streaming
			if (!bSpawnedNewActor && Bunch.bReliable && Bunch.bClose && Bunch.AtEnd())
			{
				// Do not log during replay, since this is a valid case
				if (!Connection->IsReplay())
				{
					UE_LOGF(LogNet, Verbose, "UActorChannel::ProcessBunch: SerializeNewActor received close bunch for destroyed actor. Actor: %ls, Channel: %i", *GetFullNameSafe(NewChannelActor), ChIndex);
				}

				SetChannelActor(nullptr, ESetChannelActorFlags::None);
				return;
			}

			check( !bSpawnedNewActor );
			UE_LOGF(LogNet, Warning, "UActorChannel::ProcessBunch: SerializeNewActor failed to find/spawn actor. Actor: %ls, Channel: %i, FullNetGUIDPath: %ls", *GetFullNameSafe(NewChannelActor), ChIndex, *Connection->Driver->GetNetGuidCache()->FullNetGUIDPath(ActorNetGUID));
			Broken = 1;

			if (!Connection->IsInternalAck()
#if !UE_BUILD_SHIPPING
				&& !bBlockChannelFailure
#endif
				)
			{
				FNetControlMessage<NMT_ActorChannelFailure>::Send(Connection, ChIndex);
			}
			return;
		}
		else
		{
			UE_LOGF(LogNetLifecycle, Log, "UActorChannel::ProcessBunch: Creating ChIndex: %d Actor: %ls", ChIndex, *NewChannelActor->GetName());

			if (UE::Net::bDiscardTornOffActorRPCs && NewChannelActor->GetTearOff())
			{
				UE_LOGF(LogNet, Warning, "UActorChannel::ProcessBunch: SerializeNewActor received an open bunch for a torn off actor. Actor: %ls, Channel: %i", *GetFullNameSafe(NewChannelActor), ChIndex);
				Broken = 1;

				if (!Connection->IsInternalAck()
#if !UE_BUILD_SHIPPING
					&& !bBlockChannelFailure
#endif
					)
				{
					FNetControlMessage<NMT_ActorChannelFailure>::Send(Connection, ChIndex);
				}
				return;
			}
		}

		ESetChannelActorFlags Flags = ESetChannelActorFlags::None;
		if (GSkipReplicatorForDestructionInfos != 0 && Bunch.bClose && Bunch.AtEnd())
		{
			Flags |= ESetChannelActorFlags::SkipReplicatorCreation;
		}

		UE_LOGF(LogNetTraffic, Log, "      Channel Actor %ls:", *NewChannelActor->GetFullName());
		SetChannelActor(NewChannelActor, Flags);

		NotifyActorChannelOpen(Actor, Bunch);

		RepFlags.bNetInitial = true;

		Actor->CustomTimeDilation = CustomTimeDilation;
	}
	else
	{
		UE_LOGF(LogNetTraffic, Log, "      Actor %ls:", *Actor->GetFullName() );
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool bLatestIsReplicationPaused = Bunch.bIsReplicationPaused != 0;
	if (bLatestIsReplicationPaused != IsReplicationPaused())
	{
		SetReplicationPaused(bLatestIsReplicationPaused);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Owned by connection's player?
	UNetConnection* ActorConnection = Actor->GetNetConnection();
	if (ActorConnection == Connection || (ActorConnection != NULL && ActorConnection->IsA(UChildConnection::StaticClass()) && ((UChildConnection*)ActorConnection)->Parent == Connection))
	{
		RepFlags.bNetOwner = true;
	}

	RepFlags.bIgnoreRPCs = Bunch.bIgnoreRPCs;
	RepFlags.bSkipRoleSwap = bSkipRoleSwap;

	// ----------------------------------------------
	//	Read chunks of actor content
	// ----------------------------------------------
	while ( !Bunch.AtEnd() && Connection != NULL && Connection->GetConnectionState() != USOCK_Closed )
	{
		FNetBitReader Reader( Bunch.PackageMap, 0 );

		bool bHasRepLayout = false;

		UE_NET_TRACE_NAMED_OBJECT_SCOPE(ContentBlockScope, FNetworkGUID(), Bunch, Connection->GetInTraceCollector(), ENetTraceVerbosity::Trace);

		// Read the content block header and payload
		UObject* RepObj = ReadContentBlockPayload( Bunch, Reader, bHasRepLayout );

		// Special case where we offset the events to avoid having to create a new collector for reading from the Reader
		UE_NET_TRACE_OFFSET_SCOPE(Bunch.GetPosBits() - Reader.GetNumBits(), Connection->GetInTraceCollector());

		if ( Bunch.IsError() )
		{
			if ( Connection->IsInternalAck() )
			{
				UE_LOGF( LogNet, Warning, "UActorChannel::ReceivedBunch: ReadContentBlockPayload FAILED. Bunch.IsError() == TRUE. (IsInternalAck) Breaking actor. RepObj: %ls, Channel: %i", RepObj ? *RepObj->GetFullName() : TEXT( "NULL" ), ChIndex );
				Broken = 1;
				break;
			}

			UE_LOGF( LogNet, Error, "UActorChannel::ReceivedBunch: ReadContentBlockPayload FAILED. Bunch.IsError() == TRUE. Closing connection. RepObj: %ls, Channel: %i", RepObj ? *RepObj->GetFullName() : TEXT( "NULL" ), ChIndex );

			Connection->Close(AddToAndConsumeChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockFail));

			return;
		}

		if ( Reader.GetNumBits() == 0 )
		{
			// Set the scope name
			UE_NET_TRACE_SET_SCOPE_OBJECTID(ContentBlockScope, Connection->Driver->GetNetGuidCache()->GetNetGUID(RepObj));

			// Nothing else in this block, continue on (should have been a delete or create block)
			continue;
		}

		if ( !IsValid(RepObj) )
		{
			if ( !IsValid(Actor) )
			{
				// If we couldn't find the actor, that's pretty bad, we need to stop processing on this channel
				UE_LOGF( LogNet, Warning, "UActorChannel::ProcessBunch: ReadContentBlockPayload failed to find/create ACTOR. RepObj: %ls, Channel: %i", RepObj ? *RepObj->GetFullName() : TEXT( "NULL" ), ChIndex );
				Broken = 1;
			}
			else
			{
				UE_LOGF( LogNet, Warning, "UActorChannel::ProcessBunch: ReadContentBlockPayload failed to find/create object. RepObj: %ls, Channel: %i", RepObj ? *RepObj->GetFullName() : TEXT( "NULL" ), ChIndex );
			}

			continue;	// Since content blocks separate the payload from the main stream, we can skip to the next one
		}

		TSharedRef< FObjectReplicator > & Replicator = FindOrCreateReplicator( RepObj );

		bool bHasUnmapped = false;

		if ( !Replicator->ReceivedBunch( Reader, RepFlags, bHasRepLayout, bHasUnmapped ) )
		{
			if ( Connection->IsInternalAck() )
			{
				UE_LOGF( LogNet, Warning, "UActorChannel::ProcessBunch: Replicator.ReceivedBunch failed (Ignoring because of IsInternalAck). RepObj: %ls, Channel: %i", RepObj ? *RepObj->GetFullName() : TEXT( "NULL" ), ChIndex );
				Broken = 1;
				continue;		// Don't consider this catastrophic in replays
			}

			// For now, with regular connections, consider this catastrophic, but someday we could consider supporting backwards compatibility here too
			UE_LOGF( LogNet, Error, "UActorChannel::ProcessBunch: Replicator.ReceivedBunch failed.  Closing connection. RepObj: %ls, Channel: %i", RepObj ? *RepObj->GetFullName() : TEXT( "NULL" ), ChIndex );
			Connection->Close(ENetCloseResult::ObjectReplicatorReceivedBunchFail);
			return;
		}

		// Check to see if the actor was destroyed
		// If so, don't continue processing packets on this channel, or we'll trigger an error otherwise
		// note that this is a legitimate occurrence, particularly on client to server RPCs
		if ( !IsValid(Actor) )
		{
			UE_LOGF( LogNet, VeryVerbose, "UActorChannel::ProcessBunch: Actor was destroyed during Replicator.ReceivedBunch processing" );
			// If we lose the actor on this channel, we can no longer process bunches, so consider this channel broken
			Broken = 1;
			break;
		}

		// Set the scope name now that we can lookup the NetGUID from the replicator
		UE_NET_TRACE_SET_SCOPE_OBJECTID(ContentBlockScope, Replicator->ObjectNetGUID);

		if ( bHasUnmapped )
		{
			LLM_SCOPE_BYTAG(NetDriver);
			Connection->Driver->UnmappedReplicators.Add( &Replicator.Get() );
		}
	}

#if UE_REPLICATED_OBJECT_REFCOUNTING
	TArray<TWeakObjectPtr<UObject>, TInlineAllocator<16>> ReferencesToRemove;
#endif

	for (auto RepComp = ReplicationMap.CreateIterator(); RepComp; ++RepComp)
	{
		TSharedRef<FObjectReplicator>& ObjectReplicator = RepComp.Value();
		if (!IsValid(ObjectReplicator->GetObject()))
		{
			if (!Connection->Driver->IsServer())
			{
#if UE_REPLICATED_OBJECT_REFCOUNTING
				ReferencesToRemove.Add(ObjectReplicator.Get().GetWeakObjectPtr());
#endif

				RepComp.RemoveCurrent(); // This should cause the replicator to be cleaned up as there should be no outstandings refs.
			}
			continue;
		}

		ObjectReplicator->PostReceivedBunch();
	}

#if UE_REPLICATED_OBJECT_REFCOUNTING
	if (ReferencesToRemove.Num() > 0 )
	{
		Connection->Driver->GetNetworkObjectList().RemoveMultipleSubObjectChannelReference(Actor, ReferencesToRemove, this);
	}
#endif

	// After all properties have been initialized, call PostNetInit. This should call BeginPlay() so initialization can be done with proper starting values.
	if (Actor && bSpawnedNewActor)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(PostNetInit);
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ActorChan_PostNetInit"), Stat_PostNetInit, STATGROUP_Net);
		LLM_SCOPE_BYNAME(TEXT("UObject/NetworkPostInit"));
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Actor->GetPackage(), ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Actor->GetClass(), ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET(Actor, Actor->GetClass());
		Actor->PostNetInit();
	}
}

void UActorChannel::ProcessBunch(FInBunch& Bunch)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	if (Connection && Connection->Driver && Connection->Driver->ShouldReplicateTransactionally())
	{
		static FName TransactionalWorkName("UActorChannel::ProcessBunch");
		UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [this, &Bunch]()
			{
				ProcessBunchInternal(Bunch);
			});
	}
	else
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
	{
		ProcessBunchInternal(Bunch);
	}
}

// Helper class to downgrade a non owner of an actor to simulated while replicating
class FScopedRoleDowngrade
{
public:
	FScopedRoleDowngrade( AActor* InActor, const FReplicationFlags RepFlags ) : Actor( InActor ), ActualRemoteRole( Actor->GetRemoteRole() )
	{
		// If this is actor is autonomous, and this connection doesn't own it, we'll downgrade to simulated during the scope of replication
		if ( ActualRemoteRole == ROLE_AutonomousProxy )
		{
			if ( !RepFlags.bNetOwner )
			{
				constexpr bool bIsAutonomousProxy = false; // become simulated
				constexpr bool bAllowForcePropertyCompare = false;
				Actor->SetAutonomousProxy(bIsAutonomousProxy, bAllowForcePropertyCompare);
			}
		}
	}

	~FScopedRoleDowngrade()
	{
		// Upgrade role back to autonomous proxy if needed
		if (Actor->GetRemoteRole() != ActualRemoteRole)
		{
			MARK_PROPERTY_DIRTY_FROM_NAME(AActor, RemoteRole, Actor);

			if (ActualRemoteRole == ROLE_AutonomousProxy)
			{
				constexpr bool bIsAutonomousProxy = true; // return to autonomous
				constexpr bool bAllowForcePropertyCompare = false;
				Actor->SetAutonomousProxy(bIsAutonomousProxy, bAllowForcePropertyCompare);
			}
		}
	}

private:
	AActor*			Actor;
	const ENetRole	ActualRemoteRole;
};

bool GReplicateActorTimingEnabled = false;
double GReplicateActorTimeSeconds = 0.0;
int32 GNumReplicateActorCalls = 0;

int64 UActorChannel::ReplicateActor()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	LLM_SCOPE_BYTAG(NetChannel);
	SCOPE_CYCLE_COUNTER(STAT_NetReplicateActorTime);

	check(Actor);
	check(!Closing);
	check(Connection);
	check(nullptr != Cast<UPackageMapClient>(Connection->PackageMap));

	const UWorld* const ActorWorld = Actor->GetWorld();
	ensureMsgf(ActorWorld, TEXT("ActorWorld for Actor [%s] is Null"), *GetPathNameSafe(Actor));
	if (ActorWorld == nullptr)
	{
		return 0;
	}

#if STATS || ENABLE_STATNAMEDEVENTS
	UClass* ParentNativeClass = GetParentNativeClass(Actor->GetClass());
	SCOPE_CYCLE_UOBJECT(ParentNativeClass, ParentNativeClass);
#endif

	const bool bReplay = Connection->IsReplay();
	const bool bEnableScopedCycleCounter = !bReplay && GReplicateActorTimingEnabled;
	FSimpleScopeSecondsCounter ScopedSecondsCounter(GReplicateActorTimeSeconds, bEnableScopedCycleCounter);

	if (!bReplay)
	{
		GNumReplicateActorCalls++;
	}

	// ignore hysteresis during checkpoints
	if (bIsInDormancyHysteresis && (Connection->ResendAllDataState == EResendAllDataState::None))
	{
		return 0;
	}

	// triggering replication of an Actor while already in the middle of replication can result in invalid data being sent and is therefore illegal
	if (bIsReplicatingActor)
	{
		FString Error(FString::Printf(TEXT("ReplicateActor called while already replicating! %s"), *Describe()));
		UE_LOGF(LogNet, Log, "%ls", *Error);
		ensureMsgf(false, TEXT("%s"), *Error);
		return 0;
	}

	if (bActorIsPendingKill)
	{
		// Don't need to do anything, because it should have already been logged.
		return 0;
	}

	// If our Actor is PendingKill, that's bad. It means that somehow it wasn't properly removed
	// from the NetDriver or ReplicationDriver.
	// TODO: Maybe notify the NetDriver / RepDriver about this, and have the channel close?
	if (!IsValidChecked(Actor) || Actor->IsUnreachable())
	{
		bActorIsPendingKill = true;
		ActorReplicator.Reset();
		FString Error(FString::Printf(TEXT("ReplicateActor called with PendingKill Actor! %s"), *Describe()));
		UE_LOGF(LogNet, Log, "%ls", *Error);
		ensureMsgf(false, TEXT("%s"), *Error);
		return 0;
	}

	if (bPausedUntilReliableACK)
	{
		if (NumOutRec > 0)
		{
			return 0;
		}
		bPausedUntilReliableACK = 0;
		UE_LOGF(LogNet, Verbose, "ReplicateActor: bPausedUntilReliableACK is ending now that reliables have been ACK'd. %ls", *Describe());
	}

	if (bNetReplicateOnlyBeginPlay && !IsActorReadyForReplication() && !bIsForcedSerializeFromRPC)
	{
		UE_LOGF(LogNet, Verbose, "ReplicateActor ignored since actor is not BeginPlay yet: %ls", *Describe());
		return 0;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FNetViewer>& NetViewers = ActorWorld->GetWorldSettings()->ReplicationViewers;
	bool bIsNewlyReplicationPaused = false;
	bool bIsNewlyReplicationUnpaused = false;

	if (OpenPacketId.First != INDEX_NONE && NetViewers.Num() > 0)
	{
		bool bNewPaused = false;

		const bool bOldPaused = IsReplicationPaused();

		// We were paused and still are, don't do anything.
		if (bOldPaused && bNewPaused)
		{
			return 0;
		}

		bIsNewlyReplicationUnpaused = bOldPaused && !bNewPaused;
		bIsNewlyReplicationPaused = !bOldPaused && bNewPaused;
		SetReplicationPaused(bNewPaused);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// The package map shouldn't have any carry over guids
	// Static cast is fine here, since we check above.
	UPackageMapClient* PackageMapClient = static_cast<UPackageMapClient*>(Connection->PackageMap);
	if (PackageMapClient->GetMustBeMappedGuidsInLastBunch().Num() != 0)
	{
		UE_LOGF(LogNet, Warning, "ReplicateActor: PackageMap->GetMustBeMappedGuidsInLastBunch().Num() != 0: %i: Channel: %ls", PackageMapClient->GetMustBeMappedGuidsInLastBunch().Num(), *Describe());
	}

	bool bWroteSomethingImportant = bIsNewlyReplicationUnpaused || bIsNewlyReplicationPaused;

	// Create an outgoing bunch, and skip this actor if the channel is saturated.
	FOutBunch Bunch( this, 0 );

	// Create export scope to capture NetToken exports and store them in Bunch.NetTokensPendingExport
	UE::Net::FNetTokenExportScope NetTokenExportScope(Bunch, Connection->GetDriver()->GetNetTokenStore(), Bunch.NetTokensPendingExport, "ReplicateActor");

	if( Bunch.IsError() )
	{
		return 0;
	}

	// Cache the netgroup manager so we don't have to access it for every subobject.
	DataChannelInternal::CachedNetworkSubsystem = ActorWorld->GetSubsystem<UNetworkSubsystem>();
	check(DataChannelInternal::CachedNetworkSubsystem);
	ON_SCOPE_EXIT
	{
		DataChannelInternal::CachedNetworkSubsystem = nullptr;
	};

#if UE_NET_TRACE_ENABLED
	SetTraceCollector(Bunch, UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace));
#endif

	if (bIsNewlyReplicationPaused)
	{
		Bunch.bReliable = true;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Bunch.bIsReplicationPaused = true;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CVarNetReliableDebug.GetValueOnAnyThread() > 0)
	{
		Bunch.DebugString = FString::Printf(TEXT("%.2f ActorBunch: %s"), Connection->Driver->GetElapsedTime(), *Actor->GetName() );
	}
#endif

#if UE_NET_REPACTOR_NAME_DEBUG
	// View address in memory viewer
	volatile PTRINT ActorName;

	if (bReplay && (GCVarNetActorNameDebug || GCVarNetSubObjectNameDebug))
	{
		GMetAsyncDemoNameDebugChance = HasMetAsyncDemoNameDebugChance();

		if (GCVarNetActorNameDebug && GMetAsyncDemoNameDebugChance)
		{
			ActorName = UE_NET_ALLOCA_NAME_DEBUG_BUFFER();

			StoreFullName(ActorName, Actor);
		}
	}
#endif

	FGuardValue_Bitfield(bIsReplicatingActor, true);
	FScopedRepContext RepContext(Connection, Actor);

	FReplicationFlags RepFlags;

	// Send initial stuff.
	if( OpenPacketId.First != INDEX_NONE && (Connection->ResendAllDataState == EResendAllDataState::None) )
	{
		if( !SpawnAcked && OpenAcked )
		{
			// After receiving ack to the spawn, force refresh of all subsequent unreliable packets, which could
			// have been lost due to ordering problems. Note: We could avoid this by doing it in FActorChannel::ReceivedAck,
			// and avoid dirtying properties whose acks were received *after* the spawn-ack (tricky ordering issues though).
			SpawnAcked = 1;
			for (auto RepComp = ReplicationMap.CreateIterator(); RepComp; ++RepComp)
			{
				RepComp.Value()->ForceRefreshUnreliableProperties();
			}
		}
	}
	else
	{
		if (Connection->ResendAllDataState == EResendAllDataState::SinceCheckpoint)
		{
			RepFlags.bNetInitial = !bOpenedForCheckpoint;
		}
		else
		{
			RepFlags.bNetInitial = true;
		}

		Bunch.bClose = Actor->bNetTemporary;
		Bunch.bReliable = true; // Net temporary sends need to be reliable as well to force them to retry
	}

	// Owned by connection's player?
	UNetConnection* OwningConnection = Actor->GetNetConnection();

	RepFlags.bNetOwner = (OwningConnection == Connection || (OwningConnection != nullptr && OwningConnection->IsA(UChildConnection::StaticClass()) && ((UChildConnection*)OwningConnection)->Parent == Connection));

	// ----------------------------------------------------------
	// If initial, send init data.
	// ----------------------------------------------------------

	if (RepFlags.bNetInitial && OpenedLocally)
	{
		UE_NET_TRACE_SCOPE(NewActor, Bunch, GetTraceCollector(Bunch), ENetTraceVerbosity::Trace);

		Connection->PackageMap->SerializeNewActor(Bunch, this, static_cast<AActor*&>(Actor));
		bWroteSomethingImportant = true;

		Actor->OnSerializeNewActor(Bunch);

		RepFlags.bForceInitialDirty = Bunch.bOutWantsFullInitState;
	}

	// Possibly downgrade role of actor if this connection doesn't own it
	TUniquePtr<FScopedRoleDowngrade> ScopedRoleDowngrade;
	if (Connection->Driver->CanDowngradeActorRole(Connection, Actor))
	{
		ScopedRoleDowngrade = MakeUnique<FScopedRoleDowngrade>(Actor, RepFlags);
	}

	RepFlags.bNetSimulated	= (Actor->GetRemoteRole() == ROLE_SimulatedProxy);

	if (Actor->GetRemoteRole() == ROLE_AutonomousProxy && Connection->IsProxyConnection())
	{
		RepFlags.bNetSimulated = true;
	}

	RepFlags.bRepPhysics	= Actor->GetReplicatedMovement().bRepPhysics;
	RepFlags.bReplay		= bReplay;
	RepFlags.bClientReplay	= ActorWorld->IsRecordingClientReplay();
	RepFlags.bForceInitialDirty |= Connection->IsForceInitialDirty();
	if (EnumHasAnyFlags(ActorReplicator->RepLayout->GetFlags(), ERepLayoutFlags::HasDynamicConditionProperties))
	{
		if (const FRepState* RepState = ActorReplicator->RepState.Get())
		{
			const FSendingRepState* SendingRepState = RepState->GetSendingRepState();
			if (const FRepChangedPropertyTracker* PropertyTracker = SendingRepState ? SendingRepState->RepChangedPropertyTracker.Get() : nullptr)
			{
				RepFlags.CondDynamicChangeCounter = PropertyTracker->GetDynamicConditionChangeCounter();
			}
		}
	}

	UE_LOGF(LogNetTraffic, Log, "Replicate %ls, bNetInitial: %d, bNetOwner: %d", *Actor->GetName(), RepFlags.bNetInitial, RepFlags.bNetOwner);

	FMemMark	MemMark(FMemStack::Get());	// The calls to ReplicateProperties will allocate memory on FMemStack::Get(), and use it in ::PostSendBunch. we free it below

	// ----------------------------------------------------------
	// Replicate Actor and Component properties and RPCs
	// ---------------------------------------------------

#if USE_NETWORK_PROFILER
	const uint32 ActorReplicateStartTime = GNetworkProfiler.IsTrackingEnabled() ? FPlatformTime::Cycles() : 0;
#endif

	if (!bIsNewlyReplicationPaused)
	{
		// The Actor
		{
			UE_NET_TRACE_OBJECT_SCOPE(ActorReplicator->ObjectNetGUID, Bunch, GetTraceCollector(Bunch), ENetTraceVerbosity::Trace);

			const bool bCanSkipUpdate = ActorReplicator->CanSkipUpdate(RepFlags);

			if (UE::Net::bPushModelValidateSkipUpdate || !bCanSkipUpdate)
			{
				bWroteSomethingImportant |= ActorReplicator->ReplicateProperties(Bunch, RepFlags);
			}

			ensureMsgf(!UE::Net::bPushModelValidateSkipUpdate || !bCanSkipUpdate || !bWroteSomethingImportant, TEXT("Actor wrote data but we thought it was skippable: %s"), *GetFullNameSafe(Actor));
		}

		bWroteSomethingImportant |= DoSubObjectReplication(Bunch, RepFlags);

		if (Connection->ResendAllDataState != EResendAllDataState::None)
		{
			int64 NumBitsWrote = 0;
			if (bWroteSomethingImportant)
			{
				SendBunch(&Bunch, 1);
				NumBitsWrote = Bunch.GetNumBits();
			}

			MemMark.Pop();
			NETWORK_PROFILER(GNetworkProfiler.TrackReplicateActor(Actor, RepFlags, FPlatformTime::Cycles() - ActorReplicateStartTime, Connection));
			Connection->GetDriver()->GetMetrics()->IncrementInt(UE::Net::Metric::NumReplicatedActorBytes, (NumBitsWrote + 7) >> 3);

			return NumBitsWrote;
		}

		{
			bWroteSomethingImportant |= UpdateDeletedSubObjects(Bunch);
		}
	}

	NETWORK_PROFILER(GNetworkProfiler.TrackReplicateActor(Actor, RepFlags, FPlatformTime::Cycles() - ActorReplicateStartTime, Connection));

	// -----------------------------
	// Send if necessary
	// -----------------------------

	int64 NumBitsWrote = 0;
	if (bWroteSomethingImportant)
	{
		// We must exit the collection scope to report data correctly
		FPacketIdRange PacketRange = SendBunch( &Bunch, 1 );

		if (!bIsNewlyReplicationPaused)
		{
			for (auto RepComp = ReplicationMap.CreateIterator(); RepComp; ++RepComp)
			{
				RepComp.Value()->PostSendBunch(PacketRange, Bunch.bReliable);
			}

#if NET_ENABLE_SUBOBJECT_REPKEYS
			// If there were any subobject keys pending, add them to the NakMap
			if (PendingObjKeys.Num() > 0)
			{
				// For the packet range we just sent over
				for (int32 PacketId = PacketRange.First; PacketId <= PacketRange.Last; ++PacketId)
				{
					// Get the existing set (its possible we send multiple bunches back to back and they end up on the same packet)
					FPacketRepKeyInfo &Info = SubobjectNakMap.FindOrAdd(PacketId % SubobjectRepKeyBufferSize);
					if (Info.PacketID != PacketId)
					{
						UE_LOGF(LogNetTraffic, Verbose, "ActorChannel[%d]: Clearing out PacketRepKeyInfo for new packet: %d", ChIndex, PacketId);
						Info.ObjKeys.Empty(Info.ObjKeys.Num());
					}
					Info.PacketID = PacketId;
					Info.ObjKeys.Append(PendingObjKeys);

					if (UE_LOG_ACTIVE(LogNetTraffic, Verbose))
					{
						FString VerboseString;
						for (auto KeyIt = PendingObjKeys.CreateIterator(); KeyIt; ++KeyIt)
						{
							VerboseString += FString::Printf(TEXT(" %d"), *KeyIt);
						}

						UE_LOGF(LogNetTraffic, Verbose, "ActorChannel[%d]: Sending ObjKeys: %ls", ChIndex, *VerboseString);
					}
				}
			}
#endif // NET_ENABLE_SUBOBJECT_REPKEYS

			if (Actor->bNetTemporary)
			{
				LLM_SCOPE_BYTAG(NetConnection);
				Connection->SentTemporaries.Add(Actor);
			}
		}
		NumBitsWrote = Bunch.GetNumBits();
	}

#if NET_ENABLE_SUBOBJECT_REPKEYS
	PendingObjKeys.Empty();
#endif // NET_ENABLE_SUBOBJECT_REPKEYS

	// If we evaluated everything, mark LastUpdateTime, even if nothing changed.
	LastUpdateTime = Connection->Driver->GetElapsedTime();

	MemMark.Pop();

	bForceCompareProperties = false;		// Only do this once per frame when set

	Connection->GetDriver()->GetMetrics()->IncrementInt(UE::Net::Metric::NumReplicatedActorBytes, (NumBitsWrote + 7) >> 3);
	return NumBitsWrote;
}

bool UActorChannel::DoSubObjectReplication(FOutBunch& Bunch, FReplicationFlags& OutRepFlags)
{
	bool bWroteSomethingImportant = false;

	if (Actor->IsUsingRegisteredSubObjectList())
	{
		// If the actor replicates it's subobjects and those of it's components via the list.
		bWroteSomethingImportant |= ReplicateRegisteredSubObjects(Bunch, OutRepFlags);

#if SUBOBJECT_TRANSITION_VALIDATION
		if (UE::Net::GCVarCompareSubObjectsReplicated)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Validate_ReplicatedSubObjects);
			ValidateReplicatedSubObjects();

			DataChannelInternal::bTrackReplicatedSubObjects = false;
		}
#endif
	}
	else
	{
		// Replicate the subobjects using the virtual method.
		bWroteSomethingImportant |= Actor->ReplicateSubobjects(this, &Bunch, &OutRepFlags);
	}

	AActor* TempNull(nullptr);
	SetCurrentSubObjectOwner(TempNull);

	return bWroteSomethingImportant;
}

bool UActorChannel::UpdateDeletedSubObjects(FOutBunch& Bunch)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetDeletedSubObjects);

	bool bWroteSomethingImportant = false;

	auto DeleteSubObject = [this, &bWroteSomethingImportant, &Bunch](FNetworkGUID ObjectNetGUID, const TWeakObjectPtr<UObject>& ObjectPtrToRemove, ESubObjectDeleteFlag DeleteFlag)
	{
		if (ObjectNetGUID.IsValid())
		{
			UE_NET_TRACE_SCOPE(ContentBlockForSubObjectDelete, Bunch, GetTraceCollector(Bunch), ENetTraceVerbosity::Trace);
			UE_NET_TRACE_OBJECT_SCOPE(ObjectNetGUID, Bunch, GetTraceCollector(Bunch), ENetTraceVerbosity::Trace);

			UE_LOGF(LogNetSubObject, Verbose, "NetSubObject: Sending request to %ls %ls::%ls (0x%p) NetGUID %ls", ToString(DeleteFlag), *Actor->GetName(), *GetNameSafe(ObjectPtrToRemove.GetEvenIfUnreachable()), ObjectPtrToRemove.GetEvenIfUnreachable(), *ObjectNetGUID.ToString());

			// Write a deletion content header:
			WriteContentBlockForSubObjectDelete(Bunch, ObjectNetGUID, DeleteFlag);

			bWroteSomethingImportant = true;
			Bunch.bReliable = true;
		}
		else
		{
			UE_LOGF(LogNetTraffic, Error, "Unable to write subobject delete for %ls::%ls (0x%p), object replicator has invalid NetGUID", *GetPathNameSafe(Actor), *GetNameSafe(ObjectPtrToRemove.GetEvenIfUnreachable()), ObjectPtrToRemove.GetEvenIfUnreachable());
		}
	};

	UE::Net::FDormantObjectMap* DormantObjects = Connection->GetDormantFlushedObjectsForActor(Actor);

	if (DormantObjects)
	{
		// no need to track these if they're in the replication map
		for (auto RepComp = ReplicationMap.CreateIterator(); RepComp; ++RepComp)
		{
			DormantObjects->Remove(RepComp.Value()->ObjectNetGUID);
		}
	}

#if UE_REPLICATED_OBJECT_REFCOUNTING
	TArray< TWeakObjectPtr<UObject>, TInlineAllocator<16> > SubObjectsRemoved;

	// Check if the dirty count is different from our last update
	FNetworkObjectList::FActorInvalidSubObjectView InvalidSubObjects = Connection->Driver->GetNetworkObjectList().FindActorInvalidSubObjects(Actor);
	if (InvalidSubObjects.HasInvalidSubObjects() && ChannelSubObjectDirtyCount != InvalidSubObjects.GetDirtyCount())
	{
		ChannelSubObjectDirtyCount = InvalidSubObjects.GetDirtyCount();

		for (const FNetworkObjectList::FSubObjectChannelReference& SubObjectRef : InvalidSubObjects.GetInvalidSubObjects())
		{
			ensure(SubObjectRef.Status != ENetSubObjectStatus::Active);

			// The TearOff won't be sent if the Object pointer is fully deleted once we get here.
			// This would be fixed by converting the ReplicationMap to stop using raw pointers as the key.
            // Instead the ReplicationMap iteration below should pick this deleted object and send for it to be destroyed.
			if (UObject* ObjectToRemove = SubObjectRef.SubObjectPtr.GetEvenIfUnreachable())
			{
				if (TSharedRef<FObjectReplicator>* SubObjectReplicator = ReplicationMap.Find(ObjectToRemove))
				{
					const ESubObjectDeleteFlag DeleteFlag = SubObjectRef.IsTearOff() ? ESubObjectDeleteFlag::TearOff : ESubObjectDeleteFlag::ForceDelete;
					SubObjectsRemoved.Add(SubObjectRef.SubObjectPtr);
					DeleteSubObject((*SubObjectReplicator)->ObjectNetGUID, SubObjectRef.SubObjectPtr, DeleteFlag);
					(*SubObjectReplicator)->CleanUp();
					ReplicationMap.Remove(ObjectToRemove);
				}
			}
		}

		if (!SubObjectsRemoved.IsEmpty())
		{
			Connection->Driver->GetNetworkObjectList().RemoveMultipleInvalidSubObjectChannelReference(Actor, SubObjectsRemoved, this);
			SubObjectsRemoved.Reset();
		}
	}
#endif //#if UE_REPLICATED_OBJECT_REFCOUNTING

	// Look for deleted subobjects
	for (auto RepComp = ReplicationMap.CreateIterator(); RepComp; ++RepComp)
	{
		TSharedRef<FObjectReplicator>& SubObjectReplicator = RepComp.Value();

		const TWeakObjectPtr<UObject> WeakObjPtr = SubObjectReplicator->GetWeakObjectPtr();
		if (!WeakObjPtr.IsValid())
		{
			// The only way this case would be possible is if someone tried destroying the Actor as a part of
			// a Subobject's Pre / Post replication, during Replicate Subobjects, or OnSerializeNewActor.
			// All of those are bad.
			if (!ensureMsgf(ActorReplicator.Get() != &SubObjectReplicator.Get(), TEXT("UActorChannel::ReplicateActor: Actor was deleting during replication: %s"), *Describe()))
			{
				ActorReplicator.Reset();
			}

#if UE_REPLICATED_OBJECT_REFCOUNTING
			SubObjectsRemoved.Add(WeakObjPtr);
#endif

			DeleteSubObject(SubObjectReplicator->ObjectNetGUID, WeakObjPtr, ESubObjectDeleteFlag::Destroyed);
			SubObjectReplicator->CleanUp();

			RepComp.RemoveCurrent();
		}
	}

	if (DormantObjects)
	{
		for (auto DormComp = DormantObjects->CreateConstIterator(); DormComp; ++DormComp)
		{
			const FNetworkGUID& NetGuid = DormComp.Key();
			const TWeakObjectPtr<UObject>& WeakObjPtr = DormComp.Value();

			if (!WeakObjPtr.IsValid())
			{
				DeleteSubObject(NetGuid, WeakObjPtr, ESubObjectDeleteFlag::Destroyed);
			}
		}

		Connection->ClearDormantFlushedObjectsForActor(Actor);
	}

#if UE_REPLICATED_OBJECT_REFCOUNTING
	if (!SubObjectsRemoved.IsEmpty())
	{
		Connection->Driver->GetNetworkObjectList().RemoveMultipleSubObjectChannelReference(Actor, SubObjectsRemoved, this);
	}
#endif

	return bWroteSomethingImportant;
}

bool UActorChannel::CanSubObjectReplicateToClient(
	const APlayerController* PlayerController,
	ELifetimeCondition NetCondition,
	FObjectKey SubObjectKey,
	const TStaticBitArray<COND_Max>& ConditionMap,
	const UE::Net::FNetConditionGroupManager& ConditionGroupManager)
{
	bool bCanReplicateToClient = ConditionMap[NetCondition];

	if (NetCondition == COND_NetGroup)
	{
		TArrayView<const FName> NetGroups = ConditionGroupManager.GetSubObjectNetConditionGroups(SubObjectKey);

		for (int i=0; i < NetGroups.Num() && !bCanReplicateToClient; ++i)
		{
			const FName NetGroup = NetGroups[i];

			if (UE::Net::NetGroupOwner == NetGroup)
			{
				bCanReplicateToClient = ConditionMap[COND_OwnerOnly];
			}
			else if (UE::Net::NetGroupReplay == NetGroup)
			{
				bCanReplicateToClient = ConditionMap[COND_ReplayOnly];
			}
			else
			{
				bCanReplicateToClient = PlayerController->IsMemberOfNetConditionGroup(NetGroup);
			}
		}
	}

	return bCanReplicateToClient;
}

bool UActorChannel::ReplicateRegisteredSubObjects(FOutBunch& Bunch, FReplicationFlags RepFlags)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	check(Actor->IsUsingRegisteredSubObjectList());

#if SUBOBJECT_TRANSITION_VALIDATION
	if (GCVarCompareSubObjectsReplicated || GCVarDetectDeprecatedReplicateSubObjects)
	{
		TestLegacyReplicateSubObjects(Bunch, RepFlags);

		// From here track all subobjects from the real registry that write into the bunch.
		DataChannelInternal::bTrackReplicatedSubObjects = GCVarCompareSubObjectsReplicated;
	}
#endif

	const TStaticBitArray<COND_Max> ConditionMap = UE::Net::BuildConditionMapFromRepFlags(RepFlags);

	bool bWroteSomethingImportant = false;

	// Start with the Actor's subobjects
	{
		SetCurrentSubObjectOwner(Actor);

		bWroteSomethingImportant |= WriteSubObjects(Actor, FSubObjectRegistryGetter::GetSubObjects(Actor), Bunch, RepFlags, ConditionMap);
	}

	// Now the replicated actor components
	for( const FReplicatedComponentInfo& RepComponentInfo : FSubObjectRegistryGetter::GetReplicatedComponents(Actor) )
	{
		if (CanSubObjectReplicateToClient(Connection->PlayerController, RepComponentInfo.NetCondition, RepComponentInfo.Key, ConditionMap, DataChannelInternal::CachedNetworkSubsystem->GetNetConditionGroupManager()))
		{
			UActorComponent* ReplicatedComponent = RepComponentInfo.Component;

			checkf(IsValid(ReplicatedComponent), TEXT("Found invalid replicated component (%s) registered in %s"), *GetNameSafe(ReplicatedComponent), *Actor->GetName());

			SetCurrentSubObjectOwner(ReplicatedComponent);
			if (ReplicatedComponent->IsUsingRegisteredSubObjectList())
			{
				bWroteSomethingImportant |= WriteSubObjects(ReplicatedComponent, RepComponentInfo.SubObjects, Bunch, RepFlags, ConditionMap);
			}
			else
			{
				// Replicate the component's SubObjects using the virtual method
				bWroteSomethingImportant |= ReplicatedComponent->ReplicateSubobjects(this, &Bunch, const_cast<FReplicationFlags*>(&RepFlags));
			}

			// Finally replicate the component itself at the end. SubObjects have to be created before the component on the receiving end.
			SetCurrentSubObjectOwner(Actor);
			bWroteSomethingImportant |= WriteSubObjectInBunch(ReplicatedComponent, Bunch, RepFlags);
		}
	}

	return bWroteSomethingImportant;
}


void UActorChannel::SetCurrentSubObjectOwner(AActor* SubObjectOwner)
{
	DataChannelInternal::CurrentSubObjectOwner = SubObjectOwner;
	DataChannelInternal::bIgnoreLegacyReplicateSubObject = false;
}

void UActorChannel::SetCurrentSubObjectOwner(UActorComponent* SubObjectOwner)
{
	DataChannelInternal::CurrentSubObjectOwner = SubObjectOwner;
	DataChannelInternal::bIgnoreLegacyReplicateSubObject = SubObjectOwner && SubObjectOwner->IsUsingRegisteredSubObjectList();
}

#if SUBOBJECT_TRANSITION_VALIDATION
bool UActorChannel::CanIgnoreDeprecatedReplicateSubObjects()
{
	return UE::Net::GCVarDetectDeprecatedReplicateSubObjects && DataChannelInternal::bTestingLegacyMethodForComparison;
}
#endif

bool UActorChannel::ReplicateSubobject(UObject* SubObj, FOutBunch& Bunch, FReplicationFlags RepFlags)
{
	if (!IsValid(SubObj))
	{
		return false;
	}

#if SUBOBJECT_TRANSITION_VALIDATION
	if (DataChannelInternal::bTestingLegacyMethodForComparison)
	{
		if (UE::Net::GCVarDetectDeprecatedReplicateSubObjects)
		{
			if (DataChannelInternal::CurrentSubObjectOwner == Actor)
			{
				ensureMsgf(false, TEXT("ReplicateSubObjects() should not be implemented in class %s since bReplicateUsingRegisteredSubObjectList is true. Delete the function and use AddReplicatedSubObject instead."),
					*Actor->GetClass()->GetName());
			}
			else
			{
				ensureMsgf(false, TEXT("ReplicateSubObjects() should not be implemented in class %s owned by %s (class %s) since bReplicateUsingRegisteredSubObjectList is true. Delete the function and use AddReplicatedSubObject instead."),
					*GetNameSafe(DataChannelInternal::CurrentSubObjectOwner?DataChannelInternal::CurrentSubObjectOwner->GetClass():nullptr),
					*Actor->GetName(), *Actor->GetClass()->GetName());
			}
		}
		else
		{
			DataChannelInternal::LegacySubObjectsCollected.Emplace(DataChannelInternal::FSubObjectReplicatedInfo(SubObj));
		}

		return false;
	}
#endif

	bool bWroteSomethingImportant = false;

	if (!DataChannelInternal::bIgnoreLegacyReplicateSubObject)
	{
		bWroteSomethingImportant = WriteSubObjectInBunch(SubObj, Bunch, RepFlags);
	}
	else
	{
		using namespace UE::Net;

#if SUBOBJECT_TRANSITION_VALIDATION
		// Make sure the SubObjectOwner actually has the SubObject in its list and will replicate it later.
		UActorComponent* Component = CastChecked<UActorComponent>(DataChannelInternal::CurrentSubObjectOwner);
		const bool bIsInRegistry = FSubObjectRegistryGetter::IsSubObjectInRegistry(Actor, Component, SubObj);
		ensureMsgf(bIsInRegistry, TEXT("ReplicatedSubObject %s was replicated using the legacy method but it is not in %s::%s registered list. It won't be replicated at all."),
			*SubObj->GetName(), *Actor->GetName(), *Component->GetName());
#endif
	}

	return bWroteSomethingImportant;
}

void UActorChannel::TestLegacyReplicateSubObjects(FOutBunch& Bunch, FReplicationFlags RepFlags)
{
#if SUBOBJECT_TRANSITION_VALIDATION
	// Call the legacy method but simply collect the subobjects it would have replicated
	DataChannelInternal::bTestingLegacyMethodForComparison = true;

	SetCurrentSubObjectOwner(Actor);
	Actor->ReplicateSubobjects(this, &Bunch, &RepFlags);

	DataChannelInternal::bTestingLegacyMethodForComparison = false;
#endif
}

void UActorChannel::TestLegacyReplicateSubObjects(UActorComponent* ReplicatedComponent, FOutBunch& Bunch, FReplicationFlags RepFlags)
{
#if SUBOBJECT_TRANSITION_VALIDATION
	// Call the legacy method but simply collect the subobjects it would have replicated
	DataChannelInternal::bTestingLegacyMethodForComparison = true;

	SetCurrentSubObjectOwner(ReplicatedComponent);
	ReplicatedComponent->ReplicateSubobjects(this, &Bunch, &RepFlags);

	DataChannelInternal::bTestingLegacyMethodForComparison = false;
#endif
}


bool UActorChannel::ReplicateSubobject(UActorComponent* ReplicatedComponent, FOutBunch& Bunch, FReplicationFlags RepFlags)
{
	if (!IsValid(ReplicatedComponent))
	{
		return false;
	}

	using namespace UE::Net;

	// This traps actor components using the registration list even if their owning Actor isn't using the list itself.
	if (ReplicatedComponent->IsUsingRegisteredSubObjectList() && !DataChannelInternal::bTestingLegacyMethodForComparison)
	{
#if SUBOBJECT_TRANSITION_VALIDATION
		if (GCVarCompareSubObjectsReplicated || GCVarDetectDeprecatedReplicateSubObjects)
		{
			TestLegacyReplicateSubObjects(ReplicatedComponent, Bunch, RepFlags);

			// From here track all subobjects from the real registry that write into the bunch.
			DataChannelInternal::bTrackReplicatedSubObjects = GCVarCompareSubObjectsReplicated;
		}
#endif

		bool bWroteSomethingImportant = false;

		const TStaticBitArray<COND_Max> ConditionMap = UE::Net::BuildConditionMapFromRepFlags(RepFlags);

		checkf(Actor->IsUsingRegisteredSubObjectList() == false, TEXT("This code should only be hit when an Actor that does NOT support the SubObjectList is replicating an ActorComponent that does: %s replicating %s."),
			*Actor->GetName(), *ReplicatedComponent->GetName());

		// Write all the subobjects of the component
		bWroteSomethingImportant |= WriteComponentSubObjects(ReplicatedComponent, Bunch, RepFlags, ConditionMap);

#if SUBOBJECT_TRANSITION_VALIDATION
		if (GCVarCompareSubObjectsReplicated)
		{
			ValidateReplicatedSubObjects();

			DataChannelInternal::bTrackReplicatedSubObjects = false;
		}
#endif

        // Finally write the component itself
		SetCurrentSubObjectOwner(Actor);
		bWroteSomethingImportant |= WriteSubObjectInBunch(ReplicatedComponent, Bunch, RepFlags);

		return bWroteSomethingImportant;
	}
	else
	{
		return ReplicateSubobject(StaticCast<UObject*>(ReplicatedComponent), Bunch, RepFlags);
	}
}

bool UActorChannel::WriteComponentSubObjects(UActorComponent* ReplicatedComponent, FOutBunch& Bunch, FReplicationFlags RepFlags, const TStaticBitArray<COND_Max>& ConditionMap)
{
	using namespace UE::Net;
	if (const FSubObjectRegistry* SubObjectList = FSubObjectRegistryGetter::GetSubObjectsOfActorComponent(Actor, ReplicatedComponent))
	{
		return WriteSubObjects(ReplicatedComponent, *SubObjectList, Bunch, RepFlags, ConditionMap);
	}

	return false;
}

bool UActorChannel::WriteSubObjects(UObject* SubObjectOwner, const UE::Net::FSubObjectRegistry& SubObjectList, FOutBunch& Bunch, FReplicationFlags RepFlags, const TStaticBitArray<COND_Max>& ConditionMap)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (SubObjectList.IsEmpty())
	{
		return false;
	}

	bool bWroteSomethingImportant = false;

#if UE_NET_REPACTOR_NAME_DEBUG
	// View address in memory viewer
	const bool bSubObjectNameDebug = GCVarNetSubObjectNameDebug && GMetAsyncDemoNameDebugChance && Connection->IsReplay();
	volatile PTRINT SubObjName = bSubObjectNameDebug ? UE_NET_ALLOCA_NAME_DEBUG_BUFFER() : 0;
#endif

#if UE_NET_SUBOBJECTLIST_WEAKPTR
	TArray<int32, TInlineAllocator<8>> InvalidSubObjects;
#endif

	const TArray<FSubObjectRegistry::FEntry>& SubObjects = SubObjectList.GetRegistryList();

	for (int32 Index=0; Index < SubObjects.Num(); ++Index)
	{
		const FSubObjectRegistry::FEntry& SubObjectInfo = SubObjects[Index];

		if (UObject* SubObjectToReplicate = SubObjectInfo.GetSubObject())
		{
			checkf(IsValid(SubObjectToReplicate), TEXT("Found invalid subobject (%s) registered in %s::%s"), *GetNameSafe(SubObjectToReplicate), *Actor->GetName(), *GetNameSafe(SubObjectOwner));

			if (CanSubObjectReplicateToClient(Connection->PlayerController, SubObjectInfo.NetCondition, SubObjectInfo.Key, ConditionMap, DataChannelInternal::CachedNetworkSubsystem->GetNetConditionGroupManager()))
			{
#if UE_NET_REPACTOR_NAME_DEBUG
				if (bSubObjectNameDebug)
				{
					StoreSubObjectName(SubObjName, SubObjectInfo);
				}
#endif
				bWroteSomethingImportant |= WriteSubObjectInBunch(SubObjectToReplicate, Bunch, RepFlags);
			}
		}
#if UE_NET_SUBOBJECTLIST_WEAKPTR
		else
		{
			InvalidSubObjects.Add(Index);
		}
#endif
	}

#if UE_NET_SUBOBJECTLIST_WEAKPTR
	const_cast<FSubObjectRegistry&>(SubObjectList).CleanRegistryIndexes(InvalidSubObjects);
#endif

	return bWroteSomethingImportant;
}

bool UActorChannel::ValidateReplicatedSubObjects()
{
#if SUBOBJECT_TRANSITION_VALIDATION
	bool bErrorDetected = false;
	const bool bLogAllErrors = UE::Net::CVarLogAllSubObjectComparisonErrors.GetValueOnAnyThread() != 0;

	for (int32 i=DataChannelInternal::LegacySubObjectsCollected.Num()-1; i >= 0; i--)
	{
		const DataChannelInternal::FSubObjectReplicatedInfo& Info = DataChannelInternal::LegacySubObjectsCollected[i];

		const bool bWasFound = DataChannelInternal::ReplicatedSubObjectsTracker.RemoveSingleSwap(Info, EAllowShrinking::No) != 0;

		ensureMsgf(bWasFound, TEXT("%s was not replicated by the subobject list in %s. Only by the legacy ReplicateSubObjects method."),
			*Info.Describe(Actor), *Connection->GetDriver()->NetDriverName.ToString());
		UE_CLOGF(!bWasFound && bLogAllErrors, LogNetSubObject, Warning, "%ls was not replicated by the subobject list.", *Info.Describe(Actor));

		bErrorDetected |= !bWasFound;
	}

	// Leftover subobjects in this list means we replicated more elements than the legacy method would.
	for (int32 i=DataChannelInternal::ReplicatedSubObjectsTracker.Num()-1; i >= 0; i--)
	{
		const DataChannelInternal::FSubObjectReplicatedInfo& Info = DataChannelInternal::ReplicatedSubObjectsTracker[i];

		ensureMsgf(false, TEXT("%s was replicated only in the registered subobject list in %s. Not in the legacy path."),
			*Info.Describe(Actor), *Connection->GetDriver()->NetDriverName.ToString());
		UE_CLOGF(bLogAllErrors, LogNet, Warning, "%ls was replicated only in the registered subobject list. Not in the legacy path.", *Info.Describe(Actor));
	}

	bErrorDetected |= DataChannelInternal::ReplicatedSubObjectsTracker.Num() != 0;

	DataChannelInternal::LegacySubObjectsCollected.Reset();
	DataChannelInternal::ReplicatedSubObjectsTracker.Reset();

	return bErrorDetected;
#else
	return false;
#endif
}

FString UActorChannel::Describe()
{
	if (!Actor)
	{
		return FString::Printf(TEXT("Actor: None %s"), *UChannel::Describe());
	}
	else
	{
		return FString::Printf(TEXT("[UActorChannel] Actor: %s, Role: %i, RemoteRole: %i %s"), *Actor->GetFullName(), ( int32 )Actor->GetLocalRole(), ( int32 )Actor->GetRemoteRole(), *UChannel::Describe());
	}
}


void UActorChannel::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UActorChannel* This = CastChecked<UActorChannel>(InThis);
	Super::AddReferencedObjects( This, Collector );
}

void UActorChannel::Serialize(FArchive& Ar)
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UActorChannel::Serialize");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("UActorChannel::Super", Super::Serialize(Ar));

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ReplicationMap",
			ReplicationMap.CountBytes(Ar);

			// ObjectReplicators are going to be counted by UNetDriver::Serialize AllOwnedReplicators.
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueudBunches",
			QueuedBunches.CountBytes(Ar);
			for (const FInBunch* Bunch : QueuedBunches)
			{
				if (Bunch)
				{
					Bunch->CountMemory(Ar);
				}
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingGuidResolves", PendingGuidResolves.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CreateSubObjects", GetCreatedSubObjects().CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueuedMustBeMappedGuidsInLastBunch", QueuedMustBeMappedGuidsInLastBunch.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueuedExportBunches",
			QueuedExportBunches.CountBytes(Ar);
			for (const FOutBunch* Bunch : QueuedExportBunches)
			{
				if (Bunch)
				{
					Bunch->CountMemory(Ar);
				}
			}
		);

#if NET_ENABLE_SUBOBJECT_REPKEYS
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SubobjectRepKeyMap", SubobjectRepKeyMap.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SubobjectNakMap",
			SubobjectNakMap.CountBytes(Ar);
			for (const auto& NakMapPair : SubobjectNakMap)
			{
				NakMapPair.Value.ObjKeys.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingObjKeys", PendingObjKeys.CountBytes(Ar));
#endif // NET_ENABLE_SUBOBJECT_REPKEYS

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueuedBunchObjectReferences", QueuedBunchObjectReferences.CountBytes(Ar));
	}
}

void UActorChannel::PrepareForRemoteFunction(UObject* TargetObj)
{
	// Make sure we create a replicator in case we destroy a sub object before we ever try to replicate its properties,
	// otherwise it will not be in the ReplicationMap and we'll never send the deletion to clients
	if (Connection && Connection->Driver && Connection->Driver->IsServer())
	{
		FindOrCreateReplicator(TargetObj);
	}
}

bool UActorChannel::IsActorReadyForReplication() const
{
	return (Actor ? (Actor->HasActorBegunPlay() || Actor->IsActorBeginningPlay()) : false);
}

void UActorChannel::QueueRemoteFunctionBunch( UObject* CallTarget, UFunction* Func, FOutBunch &Bunch )
{
	FindOrCreateReplicator(CallTarget).Get().QueueRemoteFunctionBunch( Func, Bunch );
}

void UActorChannel::BecomeDormant()
{
	UE_LOGF(LogNetDormancy, Verbose, "BecomeDormant: %ls", *Describe() );
	bPendingDormancy = false;
	bIsInDormancyHysteresis = false;
	Dormant = true;
	Close(EChannelCloseReason::Dormancy);
}

bool UActorChannel::ReadyForDormancy(bool suppressLogs)
{
	// We need to keep replicating the Actor and its subobjects until none of them have
	// changes, and would otherwise go Dormant normally.
	if (!bIsInDormancyHysteresis)
	{
		for (auto MapIt = ReplicationMap.CreateIterator(); MapIt; ++MapIt)
		{
			if (!MapIt.Value()->ReadyForDormancy(suppressLogs))
			{
				return false;
			}
		}
	}

	if (DormancyHysteresis > 0 && Connection && Connection->Driver)
	{
		bIsInDormancyHysteresis = true;
		const double TimePassed = Connection->Driver->GetElapsedTime() - LastUpdateTime;
		if (TimePassed < DormancyHysteresis)
		{
			return false;
		}
	}

	return true;
}

void UActorChannel::StartBecomingDormant()
{
	if (bPendingDormancy || Dormant)
	{
		return;
	}

	UE_LOGF(LogNetDormancy, Verbose, "StartBecomingDormant: %ls", *Describe() );

	for (auto MapIt = ReplicationMap.CreateIterator(); MapIt; ++MapIt)
	{
		MapIt.Value()->StartBecomingDormant();
	}
	bPendingDormancy = true;
	bIsInDormancyHysteresis = false;
	Connection->StartTickingChannel(this);
}

void UActorChannel::WriteContentBlockHeader( UObject* Obj, FNetBitWriter &Bunch, const bool bHasRepLayout )
{
	const int NumStartingBits = Bunch.GetNumBits();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GDoReplicationContextString > 0)
	{
		Connection->PackageMap->SetDebugContextString( FString::Printf(TEXT("Content Header for object: %s (Class: %s)"), *Obj->GetPathName(), *Obj->GetClass()->GetPathName() ) );
	}
#endif

	Bunch.WriteBit( bHasRepLayout ? 1 : 0 );

	// If we are referring to the actor on the channel, we don't need to send anything (except a bit signifying this)
	const bool IsActor = Obj == Actor;

	Bunch.WriteBit( IsActor ? 1 : 0 );

	if ( IsActor )
	{
		NETWORK_PROFILER( GNetworkProfiler.TrackBeginContentBlock( Obj, Bunch.GetNumBits() - NumStartingBits, Connection ) );
		return;
	}

	check(Obj);
	Bunch << Obj;
	NET_CHECKSUM(Bunch);

	if ( Connection->Driver->IsServer() )
	{
		// Only the server can tell clients to create objects, so no need for the client to send this to the server
		if ( Obj->IsNameStableForNetworking() )
		{
			Bunch.WriteBit( 1 );
		}
		else
		{
			// Not a stable name
			Bunch.WriteBit( 0 );

			// Not a destroy message
			Bunch.WriteBit( 0 );

			UClass *ObjClass = Obj->GetClass();
			Bunch << ObjClass;

			UObject* ObjOuter = Obj->GetOuter();
			// If the subobject's outer is the not the actor (and the outer is supported for networking),
			// then serialize the object's outer to rebuild the outer chain on the client.
			const bool bActorIsOuter = (ObjOuter == Actor) || (!ObjOuter->IsSupportedForNetworking());

			Bunch.WriteBit(bActorIsOuter ? 1 : 0);
			if (!bActorIsOuter)
			{
				Bunch << ObjOuter;
			}
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GDoReplicationContextString > 0)
	{
		Connection->PackageMap->ClearDebugContextString();
	}
#endif

	NETWORK_PROFILER(GNetworkProfiler.TrackBeginContentBlock(Obj, Bunch.GetNumBits() - NumStartingBits, Connection));
}

void UActorChannel::WriteContentBlockForSubObjectDelete(FOutBunch& Bunch, FNetworkGUID& GuidToDelete)
{
	WriteContentBlockForSubObjectDelete(Bunch, GuidToDelete, ESubObjectDeleteFlag::Destroyed);
}

void UActorChannel::WriteContentBlockForSubObjectDelete( FOutBunch& Bunch, FNetworkGUID& GuidToDelete, ESubObjectDeleteFlag Flag)
{
	check( Connection->Driver->IsServer() );

	const int NumStartingBits = Bunch.GetNumBits();

	// No replayout here
	Bunch.WriteBit( 0 );

	// Send a 0 bit to signify we are dealing with sub-objects
	Bunch.WriteBit( 0 );

	check( GuidToDelete.IsValid() );

	//	-Deleted object's NetGUID
	Bunch << GuidToDelete;
	NET_CHECKSUM(Bunch);	// Matches checksum in UPackageMapClient::InternalWriteObject
	NET_CHECKSUM(Bunch);	// Matches checksum in UActorChannel::ReadContentBlockHeader

	// Send a 0 bit to indicate that this is not a stably named object
	Bunch.WriteBit( 0 );

	// Send that this is a delete message
	Bunch.WriteBit( 1 );

	uint8 DeleteFlag = static_cast<uint8>(Flag);
	Bunch << DeleteFlag;

	NET_CHECKSUM(Bunch); // Matches checksum in UPackageMapClient::ReadContentBlockHeader - DeleteSubObject message

	// Since the subobject has been deleted, we don't have a valid object to pass to the profiler.
	NETWORK_PROFILER(GNetworkProfiler.TrackBeginContentBlock(nullptr, Bunch.GetNumBits() - NumStartingBits, Connection));
}

int32 UActorChannel::WriteContentBlockPayload( UObject* Obj, FNetBitWriter &Bunch, const bool bHasRepLayout, FNetBitWriter& Payload )
{
	const int32 StartHeaderBits = Bunch.GetNumBits();

	// Trace header
	{
		UE_NET_TRACE_SCOPE(ContentBlockHeader, Bunch, GetTraceCollector(Bunch), ENetTraceVerbosity::Trace);

		WriteContentBlockHeader( Obj, Bunch, bHasRepLayout );

		uint32 NumPayloadBits = Payload.GetNumBits();

		Bunch.SerializeIntPacked( NumPayloadBits );
	}

	const int32 HeaderNumBits = Bunch.GetNumBits() - StartHeaderBits;

	// Inject payload events right after header
	UE_NET_TRACE_EVENTS(GetTraceCollector(Bunch), GetTraceCollector(Payload), Bunch);

	Bunch.SerializeBits( Payload.GetData(), Payload.GetNumBits() );

	return HeaderNumBits;
}

UObject* UActorChannel::ReadContentBlockHeader(FInBunch& Bunch, bool& bObjectDeleted, bool& bOutHasRepLayout)
{
	CA_ASSUME(Connection != nullptr);

	const bool IsServer = Connection->Driver->IsServer();
	bObjectDeleted = false;

	bOutHasRepLayout = Bunch.ReadBit() != 0 ? true : false;

	if (Bunch.IsError())
	{
		UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Bunch.IsError() == true after bOutHasRepLayout. Actor: %ls", *Actor->GetName());

		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderRepLayoutFail);

		return nullptr;
	}

	const bool bIsActor = Bunch.ReadBit() != 0 ? true : false;

	if (Bunch.IsError())
	{
		UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Bunch.IsError() == true after reading actor bit. Actor: %ls", *Actor->GetName());

		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderIsActorFail);

		return nullptr;
	}

	if (bIsActor)
	{
		// If this is for the actor on the channel, we don't need to read anything else
		return Actor;
	}

	//
	// We need to handle a sub-object
	//

	// Note this heavily mirrors what happens in UPackageMapClient::SerializeNewActor
	FNetworkGUID NetGUID;
	UObject* SubObj = nullptr;

	// Manually serialize the object so that we can get the NetGUID (in order to assign it if we spawn the object here)
	Connection->PackageMap->SerializeObject(Bunch, UObject::StaticClass(), SubObj, &NetGUID);

	NET_CHECKSUM_OR_END(Bunch);

	if (Bunch.IsError())
	{
		UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Bunch.IsError() == true after SerializeObject. SubObj: %ls, Actor: %ls", SubObj ? *SubObj->GetName() : TEXT("Null"), *Actor->GetName());

		Bunch.SetError();
		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderObjFail);

		return nullptr;
	}

	if (Bunch.AtEnd())
	{
		UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Bunch.AtEnd() == true after SerializeObject. SubObj: %ls, Actor: %ls", SubObj ? *SubObj->GetName() : TEXT("Null"), *Actor->GetName());

		Bunch.SetError();
		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderPrematureEnd);

		return nullptr;
	}

	// Validate existing sub-object
	if (SubObj)
	{
		// Sub-objects can't be actors (should just use an actor channel in this case)
		if (Cast< AActor >(SubObj) != nullptr)
		{
			UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Sub-object not allowed to be actor type. SubObj: %ls, Actor: %ls", *SubObj->GetName(), *Actor->GetName());

			Bunch.SetError();
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderSubObjectActor);

			return nullptr;
		}

		// Sub-objects must reside within their actor parents
		UActorComponent* Component = Cast<UActorComponent>(SubObj);
		if (Component && !SubObj->IsIn(Actor))
		{
			UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Actor component not in parent actor. SubObj: %ls, Actor: %ls", *SubObj->GetFullName(), *Actor->GetFullName());

			if (IsServer)
			{
				Bunch.SetError();
				AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderBadParent);

				return nullptr;
			}
		}
	}

	if (IsServer)
	{
		// The server should never need to create sub objects
		if (!SubObj)
		{
			UE_LOGF(LogNetTraffic, Error, "ReadContentBlockHeader: Client attempted to create sub-object. Actor: %ls", *Actor->GetName());

			Bunch.SetError();
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderInvalidCreate);

			return nullptr;
		}

		return SubObj;
	}

	const bool bStablyNamed = Bunch.ReadBit() != 0 ? true : false;

	if (Bunch.IsError())
	{
		UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Bunch.IsError() == true after reading stably named bit. Actor: %ls", *Actor->GetName());

		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderStablyNamedFail);

		return nullptr;
	}

	if (bStablyNamed)
	{
		// If this is a stably named sub-object, we shouldn't need to create it. Don't raise a bunch error though because this may happen while a level is streaming out.
		if (!SubObj)
		{
			// (ignore though if this is for replays)
			if (!Connection->IsInternalAck())
			{
				UE_LOGF(LogNetTraffic, Log, "ReadContentBlockHeader: Stably named sub-object not found. Its level may have streamed out. Component: %ls, Actor: %ls", *Connection->Driver->GetNetGuidCache()->FullNetGUIDPath(NetGUID), *Actor->GetName());
			}

			return nullptr;
		}

		return SubObj;
	}

	bool bDeleteSubObject = false;
	bool bSerializeClass = true;
	ESubObjectDeleteFlag DeleteFlag = ESubObjectDeleteFlag::Destroyed;

	if (Bunch.EngineNetVer() >= FEngineNetworkCustomVersion::SubObjectDestroyFlag)
	{
		const bool bIsDestroyMessage = Bunch.ReadBit() != 0 ? true : false;

		if (bIsDestroyMessage)
		{
			bDeleteSubObject = true;
			bSerializeClass = false;

			uint8 ReceivedDestroyFlag;
			Bunch << ReceivedDestroyFlag;
			DeleteFlag = static_cast<ESubObjectDeleteFlag>(ReceivedDestroyFlag);

			NET_CHECKSUM(Bunch);
		}
	}
	else
	{
		bSerializeClass = true;
	}


	UObject* SubObjClassObj = nullptr;
	if (bSerializeClass)
	{
		// Serialize the class in case we have to spawn it.
		// Manually serialize the object so that we can get the NetGUID (in order to assign it if we spawn the object here)
		FNetworkGUID ClassNetGUID;
		Connection->PackageMap->SerializeObject(Bunch, UObject::StaticClass(), SubObjClassObj, &ClassNetGUID);

		// When ClassNetGUID is empty it means the sender requested to delete the subobject
		bDeleteSubObject = !ClassNetGUID.IsValid();
	}


	if (bDeleteSubObject)
	{
		if (SubObj)
		{
			// Unmap this object so we can remap it if it becomes relevant again in the future
			MoveMappedObjectToUnmapped(SubObj);

			// Stop tracking this sub-object
			GetCreatedSubObjects().Remove(SubObj);

			if (Connection != nullptr && Connection->Driver != nullptr)
			{
				Connection->Driver->NotifySubObjectDestroyed(SubObj);
			}

			UE_LOGF(LogNetSubObject, Verbose, "NetSubObject: Client received request to %ls %ls::%ls (0x%p) NetGuid %ls", ToString(DeleteFlag), *Actor->GetName(), *GetNameSafe(SubObj), SubObj, *NetGUID.ToString());

			if (DeleteFlag != ESubObjectDeleteFlag::TearOff)
			{
				Actor->OnSubobjectDestroyFromReplication(SubObj);

				SubObj->PreDestroyFromReplication();
				SubObj->MarkAsGarbage();
			}
		}

		bObjectDeleted = true;
		return nullptr;
	}

	UClass* SubObjClass = Cast< UClass >(SubObjClassObj);

	if (!SubObjClass)
	{
		UE_LOGF(LogNetTraffic, Warning, "UActorChannel::ReadContentBlockHeader: Unable to read sub-object class. Actor: %ls", *Actor->GetName());

		// Valid NetGUID but no class was resolved - this is an error
		if (!SubObj)
		{
			// (unless we're using replays, which could be backwards compatibility kicking in)
			if (!Connection->IsInternalAck())
			{
				UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Unable to read sub-object class (SubObj == NULL). Actor: %ls", *Actor->GetName());

				Bunch.SetError();
				AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderNoSubObjectClass);
				return nullptr;
			}
		}
	}
	else
	{
		if (SubObjClass == UObject::StaticClass())
		{
			UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: SubObjClass == UObject::StaticClass(). Actor: %ls", *Actor->GetName());

			Bunch.SetError();
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderUObjectSubObject);

			return nullptr;
		}

		if (SubObjClass->IsChildOf(AActor::StaticClass()))
		{
			UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Sub-object cannot be actor class. Actor: %ls", *Actor->GetName());

			Bunch.SetError();
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderAActorSubObject);

			return nullptr;
		}
	}

	UObject* ObjOuter = Actor;
	if (Bunch.EngineNetVer() >= FEngineNetworkCustomVersion::SubObjectOuterChain)
	{
		const bool bActorIsOuter = Bunch.ReadBit() != 0;

		if (Bunch.IsError())
		{
			UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Bunch.IsError() == true after reading actor is outer bit. Actor: %ls", *Actor->GetName());
			return nullptr;
		}

		if (!bActorIsOuter)
		{
			// If the channel's actor isn't the object's outer, serialize the object's outer here
			Bunch << ObjOuter;

			if (Bunch.IsError())
			{
				UE_LOGF(LogNetTraffic, Error, "UActorChannel::ReadContentBlockHeader: Bunch.IsError() == true after serializing the subobject's outer. Actor: %ls", *Actor->GetName());
				return nullptr;
			}

			if (ObjOuter == nullptr && SubObj == nullptr)
			{
				// When replicating subobjects, the engine will by default replicate lower subobjects before their outers, using the owning actor as the outer on the client.
				// To have a chain of subobjects maintain their correct outers on the client, the subobjects should be replicated top-down, so the outers are received before any subobjects they own.
				UE_LOGF(LogNetTraffic, Log, "UActorChannel::ReadContentBlockHeader: Unable to serialize subobject's outer, using owning actor as outer instead. Class: %ls, Actor: %ls", *GetNameSafe(SubObjClass), *GetNameSafe(Actor));

				ObjOuter = Actor;
			}
		}
	}

	// in this case we may still have needed to serialize the outer above, so check again now
	if ((SubObjClass == nullptr) && (SubObj == nullptr) && Connection->IsInternalAck())
	{
		return nullptr;
	}

	if (!SubObj)
	{
		check( !IsServer );

		// Construct the sub-object
		UE_LOGF( LogNetTraffic, Log, "UActorChannel::ReadContentBlockHeader: Instantiating sub-object. Class: %ls, Actor: %ls, Outer: %ls", *GetNameSafe(SubObjClass), *Actor->GetName(), *ObjOuter->GetName() );

		SubObj = NewObject< UObject >(ObjOuter, SubObjClass);

		// Sanity check some things
		checkf(SubObj != nullptr, TEXT("UActorChannel::ReadContentBlockHeader: Subobject is NULL after instantiating. Class: %s, Actor %s"), *GetNameSafe(SubObjClass), *Actor->GetName());
		checkf(SubObj->IsIn(ObjOuter), TEXT("UActorChannel::ReadContentBlockHeader: Subobject is not in Outer. SubObject: %s, Actor: %s, Outer: %s"), *SubObj->GetName(), *Actor->GetName(), *ObjOuter->GetName());
		checkf(Cast< AActor >(SubObj) == nullptr, TEXT("UActorChannel::ReadContentBlockHeader: Subobject is an Actor. SubObject: %s, Actor: %s"), *SubObj->GetName(), *Actor->GetName());

		// Register the component guid
		Connection->Driver->GetNetGuidCache()->RegisterNetGUID_Client(NetGUID, SubObj);

		// Notify actor that we created a component from replication
		Actor->OnSubobjectCreatedFromReplication( SubObj );

		// Track which sub-object guids we are creating
		GetCreatedSubObjects().Add( SubObj );

		// Add this sub-object to the ImportedNetGuids list so we can possibly map this object if needed
		if (ensureMsgf(NetGUID.IsValid(), TEXT("Channel tried to add an invalid GUID to the import list: %s"), *Describe()))
		{
			LLM_SCOPE_BYTAG(GuidCache);
			Connection->Driver->GetNetGuidCache()->ImportedNetGuids.Add( NetGUID );
		}
	}

	return SubObj;
}

UObject* UActorChannel::ReadContentBlockPayload( FInBunch &Bunch, FNetBitReader& OutPayload, bool& bOutHasRepLayout )
{
	const int32 StartHeaderBits = Bunch.GetPosBits();
	bool bObjectDeleted = false;
	UObject* RepObj = ReadContentBlockHeader( Bunch, bObjectDeleted, bOutHasRepLayout );

	if ( Bunch.IsError() )
	{
		UE_LOGF( LogNet, Error, "UActorChannel::ReadContentBlockPayload: ReadContentBlockHeader FAILED. Bunch.IsError() == TRUE. Closing connection. RepObj: %ls, Channel: %i", RepObj ? *RepObj->GetFullName() : TEXT( "NULL" ), ChIndex );

		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockHeaderFail);

		return nullptr;
	}

	if ( bObjectDeleted )
	{
		OutPayload.SetData( Bunch, 0 );

		// Nothing else in this block, continue on
		return nullptr;
	}

	uint32 NumPayloadBits = 0;
	Bunch.SerializeIntPacked( NumPayloadBits );

	UE_NET_TRACE(ContentBlockHeader, Connection->GetInTraceCollector(), StartHeaderBits, Bunch.GetPosBits(), ENetTraceVerbosity::Trace);

	if ( Bunch.IsError() )
	{
		UE_LOGF( LogNet, Error, "UActorChannel::ReceivedBunch: Read NumPayloadBits FAILED. Bunch.IsError() == TRUE. Closing connection. RepObj: %ls, Channel: %i", RepObj ? *RepObj->GetFullName() : TEXT( "NULL" ), ChIndex );

		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::ContentBlockPayloadBitsFail);

		return nullptr;
	}

	OutPayload.SetData( Bunch, NumPayloadBits );

	return RepObj;
}

int32 UActorChannel::WriteFieldHeaderAndPayload( FNetBitWriter& Bunch, const FClassNetCache* ClassCache, const FFieldNetCache* FieldCache, FNetFieldExportGroup* NetFieldExportGroup, FNetBitWriter& Payload, bool bIgnoreInternalAck )
{
	const int32 NumOriginalBits = Bunch.GetNumBits();

	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(FieldCache->Field.GetFName(), Bunch, GetTraceCollector(Bunch), ENetTraceVerbosity::Trace);

	NET_CHECKSUM( Bunch );

	if ( Connection->IsInternalAck() && !bIgnoreInternalAck )
	{
		check( NetFieldExportGroup != nullptr );

		const int32 NetFieldExportHandle = NetFieldExportGroup->FindNetFieldExportHandleByChecksum( FieldCache->FieldChecksum );

		check( NetFieldExportHandle >= 0 );

		CastChecked<UPackageMapClient>(Connection->PackageMap)->TrackNetFieldExport( NetFieldExportGroup, NetFieldExportHandle );

		check( NetFieldExportHandle < NetFieldExportGroup->NetFieldExports.Num() );

		Bunch.WriteIntWrapped( NetFieldExportHandle, FMath::Max( NetFieldExportGroup->NetFieldExports.Num(), 2 ) );
	}
	else
	{
		const int32 MaxFieldNetIndex = ClassCache->GetMaxIndex() + 1;

		check( FieldCache->FieldNetIndex < MaxFieldNetIndex );

		Bunch.WriteIntWrapped( FieldCache->FieldNetIndex, MaxFieldNetIndex );
	}

	uint32 NumPayloadBits = Payload.GetNumBits();

	Bunch.SerializeIntPacked( NumPayloadBits );

	UE_NET_TRACE(FieldHeader, GetTraceCollector(Bunch), NumOriginalBits, Bunch.GetNumBits(), ENetTraceVerbosity::Trace);

	// Inject trace data from payload stream
	UE_NET_TRACE_EVENTS(GetTraceCollector(Bunch), GetTraceCollector(Payload), Bunch);

	Bunch.SerializeBits( Payload.GetData(), NumPayloadBits );

	return Bunch.GetNumBits() - NumOriginalBits;
}

bool UActorChannel::ReadFieldHeaderAndPayload( UObject* Object, const FClassNetCache* ClassCache, FNetFieldExportGroup* NetFieldExportGroup, FNetBitReader& Bunch, const FFieldNetCache** OutField, FNetBitReader& OutPayload ) const
{
	*OutField = nullptr;

	if ( Bunch.GetBitsLeft() == 0 )
	{
		return false;	// We're done
	}

	const int64 HeaderBitPos = Bunch.GetPosBits();

	NET_CHECKSUM( Bunch );

	if ( Connection->IsInternalAck() )
	{
		if ( NetFieldExportGroup == nullptr )
		{
			UE_LOGF( LogNet, Warning, "ReadFieldHeaderAndPayload: NetFieldExportGroup was null. Object: %ls", *Object->GetFullName() );
			Bunch.SetError();
			return false;
		}

		const int32 NetFieldExportHandle = Bunch.ReadInt( FMath::Max( NetFieldExportGroup->NetFieldExports.Num(), 2 ) );

		if ( Bunch.IsError() )
		{
			UE_LOGF( LogNet, Error, "ReadFieldHeaderAndPayload: Error reading NetFieldExportHandle. Object: %ls", *Object->GetFullName() );
			return false;
		}

		if ( !ensure( NetFieldExportHandle < NetFieldExportGroup->NetFieldExports.Num() ) )
		{
			UE_LOGF( LogRep, Error, "ReadFieldHeaderAndPayload: NetFieldExportHandle too large. Object: %ls, NetFieldExportHandle: %i", *Object->GetFullName(), NetFieldExportHandle );
			Bunch.SetError();
			return false;
		}

		const FNetFieldExport& NetFieldExport = NetFieldExportGroup->NetFieldExports[NetFieldExportHandle];

		if ( !ensure( NetFieldExport.CompatibleChecksum != 0 ) )
		{
			UE_LOGF( LogNet, Error, "ReadFieldHeaderAndPayload: NetFieldExport.CompatibleChecksum was 0. Object: %ls, Property: %ls, NetFieldExportHandle: %i", *Object->GetFullName(), *NetFieldExport.ExportName.ToString(), NetFieldExportHandle );
			Bunch.SetError();
			return false;
		}

		*OutField = ClassCache->GetFromChecksum( NetFieldExport.CompatibleChecksum );

		if ( *OutField == NULL )
		{
			if ( !NetFieldExport.bIncompatible )
			{
				UE_LOGF( LogNet, Warning, "ReadFieldHeaderAndPayload: GetFromChecksum failed (NetBackwardsCompatibility). Object: %ls, Property: %ls", *Object->GetFullName(), *NetFieldExport.ExportName.ToString() );
				NetFieldExport.bIncompatible = true;
			}
		}
	}
	else
	{
		const int32 RepIndex = Bunch.ReadInt( ClassCache->GetMaxIndex() + 1 );

		if ( Bunch.IsError() )
		{
			UE_LOGF( LogRep, Error, "ReadFieldHeaderAndPayload: Error reading RepIndex. Object: %ls", *Object->GetFullName() );

			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::FieldHeaderRepIndex);

			return false;
		}

		if ( RepIndex > ClassCache->GetMaxIndex() )
		{
			UE_LOGF( LogRep, Error, "ReadFieldHeaderAndPayload: RepIndex too large. Object: %ls", *Object->GetFullName() );

			Bunch.SetError();
			AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::FieldHeaderBadRepIndex);

			return false;
		}

		*OutField = ClassCache->GetFromIndex( RepIndex );

		if ( *OutField == NULL )
		{
			UE_LOGF( LogNet, Warning, "ReadFieldHeaderAndPayload: GetFromIndex failed. Object: %ls", *Object->GetFullName() );
		}
	}

	uint32 NumPayloadBits = 0;
	Bunch.SerializeIntPacked( NumPayloadBits );

	UE_NET_TRACE(FieldHeader, Connection->GetInTraceCollector(), HeaderBitPos, Bunch.GetPosBits(), ENetTraceVerbosity::Trace);

	if ( Bunch.IsError() )
	{
		UE_LOGF( LogNet, Error, "ReadFieldHeaderAndPayload: Error reading numbits. Object: %ls, OutField: %ls", *Object->GetFullName(), ( *OutField && (*OutField)->Field ) ? *(*OutField)->Field.GetName() : TEXT( "NULL" ) );

		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::FieldHeaderPayloadBitsFail);

		return false;
	}

	OutPayload.SetData( Bunch, NumPayloadBits );

	if ( Bunch.IsError() )
	{
		UE_LOGF( LogNet, Error, "ReadFieldHeaderAndPayload: Error reading payload. Object: %ls, OutField: %ls", *Object->GetFullName(), ( *OutField && (*OutField)->Field ) ? *(*OutField)->Field.GetName() : TEXT( "NULL" ) );

		AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::FieldPayloadFail);

		return false;
	}

	return true;		// More to read
}

FNetFieldExportGroup* UActorChannel::GetOrCreateNetFieldExportGroupForClassNetCache( const UObject* Object )
{
	if ( !Connection->IsInternalAck() )
	{
		return nullptr;
	}

	check( Object );

	UClass* ObjectClass = Object->GetClass();

	checkf( ObjectClass, TEXT( "ObjectClass is null. ObjectName: %s" ), *GetNameSafe( Object ) );
	checkf( ObjectClass->IsValidLowLevelFast(), TEXT( "ObjectClass is invalid. ObjectName: %s" ), *GetNameSafe( Object ) );

	UPackageMapClient* PackageMapClient = ( ( UPackageMapClient* )Connection->PackageMap );

	FString NetFieldExportGroupName = ObjectClass->GetPathName();
	GEngine->NetworkRemapPath(Connection, NetFieldExportGroupName, false);
	NetFieldExportGroupName += ClassNetCacheSuffix;

	TSharedPtr< FNetFieldExportGroup > NetFieldExportGroup = PackageMapClient->GetNetFieldExportGroup( NetFieldExportGroupName );

	if ( !NetFieldExportGroup.IsValid() )
	{
		const FClassNetCache* ClassCache = Connection->Driver->NetCache->GetClassNetCache( ObjectClass );

		NetFieldExportGroup = TSharedPtr< FNetFieldExportGroup >( new FNetFieldExportGroup() );

		NetFieldExportGroup->PathName = NetFieldExportGroupName;

		int32 CurrentHandle = 0;

		for ( const FClassNetCache* C = ClassCache; C; C = C->GetSuper() )
		{
			const TArray< FFieldNetCache >& Fields = C->GetFields();

			for (const FFieldNetCache& NetField : Fields)
			{
				const FFieldVariant& Field = NetField.Field;
				FProperty* Property = CastField< FProperty >( Field.ToField() );

				const bool bIsCustomDeltaProperty	= Property && IsCustomDeltaProperty( Property );
				const bool bIsFunction				= Cast< UFunction >( Field.ToUObject() ) != nullptr;

				if ( !bIsCustomDeltaProperty && !bIsFunction )
				{
					continue;	// We only care about net fields that aren't in a rep layout
				}

				NetFieldExportGroup->NetFieldExports.Emplace( CurrentHandle++, NetField.FieldChecksum, Field.GetFName() );
			}
		}

		PackageMapClient->AddNetFieldExportGroup( NetFieldExportGroupName, NetFieldExportGroup );
	}

	return NetFieldExportGroup.Get();
}

FNetFieldExportGroup* UActorChannel::GetNetFieldExportGroupForClassNetCache(UClass* ObjectClass)
{
	if (!Connection->IsInternalAck())
	{
		return nullptr;
	}

	FString NetFieldExportGroupName;

	if (Connection->GetNetworkCustomVersion(FEngineNetworkCustomVersion::Guid) < FEngineNetworkCustomVersion::ClassNetCacheFullName)
	{
		NetFieldExportGroupName = ObjectClass->GetName() + ClassNetCacheSuffix;
	}
	else
	{
		NetFieldExportGroupName = ObjectClass->GetPathName();
		GEngine->NetworkRemapPath(Connection, NetFieldExportGroupName, true);
		NetFieldExportGroupName += ClassNetCacheSuffix;
	}

	UPackageMapClient* PackageMapClient = CastChecked<UPackageMapClient>(Connection->PackageMap);

	TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = PackageMapClient->GetNetFieldExportGroup(NetFieldExportGroupName);

	return NetFieldExportGroup.Get();
}

FObjectReplicator & UActorChannel::GetActorReplicationData()
{
	// TSharedPtr will do a check before dereference, so no need to explicitly check here.
	return *ActorReplicator;
}

TSharedRef<FObjectReplicator>* UActorChannel::FindReplicator(UObject* Obj)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FindReplicator(Obj, nullptr);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TSharedRef<FObjectReplicator>* UActorChannel::FindReplicator(UObject* Obj, bool* bOutFoundInvalid)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(Stat_ActorChanFindOrCreateRep, GUseDetailedScopeCounters);

	// First, try to find it on the channel replication map
	TSharedRef<FObjectReplicator>* ReplicatorRefPtr = ReplicationMap.Find( Obj );
	if (ReplicatorRefPtr)
	{
		if (!ReplicatorRefPtr->Get().GetWeakObjectPtr().IsValid())
		{
#if UE_REPLICATED_OBJECT_REFCOUNTING
			Connection->Driver->GetNetworkObjectList().RemoveSubObjectChannelReference(Actor, ReplicatorRefPtr->Get().GetWeakObjectPtr(), this);
#endif

			ReplicatorRefPtr = nullptr;
			ReplicationMap.Remove(Obj);

			if (bOutFoundInvalid)
			{
				*bOutFoundInvalid = true;
			}
		}
	}

	return ReplicatorRefPtr;
}

TSharedRef<FObjectReplicator>& UActorChannel::CreateReplicator(UObject* Obj)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// We always want to reuse the dormant replicator if it was stored in the connection previously.
	return CreateReplicator(Obj, true);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TSharedRef<FObjectReplicator>& UActorChannel::CreateReplicator(UObject* Obj, bool bCheckDormantReplicators)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(Stat_ActorChanFindOrCreateRep, GUseDetailedScopeCounters);

	// Try to find in the dormancy map if desired
	TSharedPtr<FObjectReplicator> NewReplicator;
	if (bCheckDormantReplicators)
	{
		NewReplicator = Connection->FindAndRemoveDormantReplicator(Actor, Obj);

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
		if (NewReplicator.IsValid())
		{
			// If this is a subobject (and not the main actor) swap the reference from the NetConnection to this channel
			if (Obj != Actor)
			{
				if (UNetDriver* NetDriver = Connection->GetDriver())
				{
					NetDriver->GetNetworkObjectList().SwapReferenceForDormancy(Actor, Obj, Connection, this);
				}
			}
		}
#endif
	}
	// Failsafe thats makes sure to discard any dormant replicator until we completely delete the option to ignore them.
	else
	{
		// Discard dormant replicator if there was one that we're not using
		Connection->RemoveDormantReplicator(Actor, Obj);
	}

	// Check if we found it and that it is has a valid object
	if(NewReplicator.IsValid())
	{
		UE_LOGF(LogNetTraffic, Log, "CreateReplicator - Found existing replicator for (0x%p) %ls - Replicator (0x%p)", Obj , *Obj->GetName(), NewReplicator.Get());
	}
	else
	{
		// Didn't find one, need to create
		NewReplicator = Connection->CreateReplicatorForNewActorChannel(Obj);

		UE_LOGF(LogNetTraffic, Log, "CreateReplicator - creating new replicator for (0x%p) %ls - Replicator (0x%p)", Obj, *Obj->GetName(), NewReplicator.Get());

#if UE_REPLICATED_OBJECT_REFCOUNTING
		// If this is a subobject (and not the main actor) add a reference
		if (Obj != Actor)
		{
			if (UNetDriver* NetDriver = Connection->GetDriver())
			{
				NetDriver->GetNetworkObjectList().AddSubObjectChannelReference(Actor, Obj, this);
			}
		}
#endif
	}

	// Add to the replication map
	TSharedRef<FObjectReplicator>& NewRef = ReplicationMap.Add(Obj, NewReplicator.ToSharedRef());

	// Start replicating with this replicator
	NewRef->StartReplicating(this);
	return NewRef;
}

TSharedRef<FObjectReplicator>& UActorChannel::FindOrCreateReplicator(UObject* Obj, bool* bOutCreated)
{
	TSharedRef<FObjectReplicator>* ReplicatorRefPtr = FindReplicator(Obj);

	// This should only be false if we found the replicator in the ReplicationMap
	// If we pickup the replicator from the DormantReplicatorSet we treat it as it has been created.
	if (bOutCreated != nullptr)
	{
		*bOutCreated = (ReplicatorRefPtr == nullptr);
	}

	if (!ReplicatorRefPtr)
	{
		ReplicatorRefPtr = &CreateReplicator(Obj);
	}

	return *ReplicatorRefPtr;
}

bool UActorChannel::ObjectHasReplicator(const TWeakObjectPtr<UObject>& Obj) const
{
	const TSharedRef<FObjectReplicator>* ReplicatorRefPtr = ReplicationMap.Find(Obj.Get());
	return ReplicatorRefPtr != nullptr && Obj == ReplicatorRefPtr->Get().GetWeakObjectPtr();
}

#if NET_ENABLE_SUBOBJECT_REPKEYS
bool UActorChannel::KeyNeedsToReplicate(int32 ObjID, int32 RepKey)
{
	if (bPendingDormancy)
	{
		// Return true to keep the channel from being stuck unable to go dormant until the key changes
		return true;
	}

	int32 &MapKey = SubobjectRepKeyMap.FindOrAdd(ObjID);
	if (MapKey == RepKey)
	{
		return false;
	}

	MapKey = RepKey;
	PendingObjKeys.Add(ObjID);
	return true;
}
#endif // NET_ENABLE_SUBOBJECT_REPKEYS

void UActorChannel::AddedToChannelPool()
{
	Super::AddedToChannelPool();

	check(!ActorReplicator);
	check(ReplicationMap.Num() == 0);
	check(QueuedBunches.Num() == 0);
	check(PendingGuidResolves.Num() == 0);
	check(QueuedBunchObjectReferences.Num() == 0);
	check(QueuedMustBeMappedGuidsInLastBunch.Num() == 0);
	check(QueuedExportBunches.Num() == 0);

	ReleaseReferences(false);

	ActorNetGUID = FNetworkGUID();
	CustomTimeDilation = 0;
	RelevantTime = 0;
	LastUpdateTime = 0;
	SpawnAcked = false;
	bForceCompareProperties = false;
	bIsReplicatingActor = false;
	bActorIsPendingKill = false;
	bSkipRoleSwap = false;
	bClearRecentActorRefs = true;
	ChannelSubObjectDirtyCount = 0;
	QueuedBunchStartTime = 0;
	bSuppressQueuedBunchWarningsDueToHitches = false;
#if !UE_BUILD_SHIPPING
	bBlockChannelFailure = false;
#endif
	QueuedCloseReason = EChannelCloseReason::Destroyed;

#if NET_ENABLE_SUBOBJECT_REPKEYS
	SubobjectRepKeyMap.Empty();
	SubobjectNakMap.Empty();
	PendingObjKeys.Empty();
#endif // NET_ENABLE_SUBOBJECT_REPKEYS
}

bool UActorChannel::WriteSubObjectInBunch(UObject* Obj, FOutBunch& Bunch, FReplicationFlags RepFlags)
{
	// This matches the check in FObjectReplicator::InitWithObject if we do not early out here then the connection will be improperly initialized.
	TWeakObjectPtr<UObject> WeakObjectPtr = Obj;
	if (!IsValid(Obj) || !WeakObjectPtr.IsValid())
	{
		return false;
	}

	check(DataChannelInternal::bTestingLegacyMethodForComparison == false);

    SCOPE_CYCLE_UOBJECT(ActorChannelRepSubObj, Obj);

#if SUBOBJECT_TRANSITION_VALIDATION
	if (DataChannelInternal::bTrackReplicatedSubObjects)
	{
		DataChannelInternal::ReplicatedSubObjectsTracker.Emplace(DataChannelInternal::FSubObjectReplicatedInfo(Obj));
	}
#endif

    if (RepFlags.bUseCustomSubobjectReplication)
	{
		return ReplicateSubobjectCustom(Obj, Bunch, RepFlags);
	}

	TSharedRef<FObjectReplicator>* FoundReplicator = FindReplicator(Obj);
	const bool bFoundReplicator = (FoundReplicator != nullptr);

	// Special case for replay checkpoints where the replicator could already exist but we still need to consider this a new object even if it has an empty layout
	const bool bNewToReplay = (Connection->ResendAllDataState == EResendAllDataState::SinceOpen)
		|| ((Connection->ResendAllDataState == EResendAllDataState::SinceCheckpoint) && (!bFoundReplicator || (*FoundReplicator)->IsDirtyForReplay()));

	const bool bCanSkipUpdate = bFoundReplicator && !bNewToReplay && (*FoundReplicator)->CanSkipUpdate(RepFlags);

	if (!UE::Net::bPushModelValidateSkipUpdate && bCanSkipUpdate)
	{
		return false;
	}

	TWeakObjectPtr<UObject> WeakObj(Obj);

	// Hack for now: subobjects are SupportsObject==false until they are replicated via ::ReplicateSUbobject, and then we make them supported
	// here, by forcing the packagemap to give them a NetGUID.
	//
	// Once we can lazily handle unmapped references on the client side, this can be simplified.
	if ( !Connection->Driver->GetNetGuidCache()->SupportsObject( Obj, &WeakObj ) )
	{
		Connection->Driver->GetNetGuidCache()->AssignNewNetGUID_Server( Obj );	//Make sure it gets a NetGUID so that it is now 'supported'
	}

	FReplicationFlags ObjRepFlags = RepFlags;
	TSharedRef<FObjectReplicator>& ObjectReplicator = !bFoundReplicator ? CreateReplicator(Obj) : *FoundReplicator;

	const bool bIsNewSubObject = (ObjectReplicator->bSentSubObjectCreation == false) || bNewToReplay;

	if (bIsNewSubObject)
	{
		// This is the first time replicating this subobject
		// This bunch should be reliable and we should always return true
		// even if the object properties did not diff from the CDO
		// (this will ensure the content header chunk is sent which is all we care about
		// to spawn this on the client).
		Bunch.bReliable = true;
		ObjectReplicator->bSentSubObjectCreation = true;

		if (UE::Net::bEnableNetInitialSubObjects)
		{
			ObjRepFlags.bNetInitial = true;
		}
	}
	UE_NET_TRACE_OBJECT_SCOPE(ObjectReplicator->ObjectNetGUID, Bunch, GetTraceCollector(Bunch), ENetTraceVerbosity::Trace);
	bool bWroteSomething = ObjectReplicator.Get().ReplicateProperties(Bunch, ObjRepFlags);
	if (bIsNewSubObject && !bWroteSomething)
	{
		// Write empty payload to force object creation
		FNetBitWriter EmptyPayload;
		WriteContentBlockPayload( Obj, Bunch, false, EmptyPayload );
		bWroteSomething = true;
	}

	ensureMsgf(!UE::Net::bPushModelValidateSkipUpdate || !bCanSkipUpdate || !bWroteSomething, TEXT("Subobject wrote data but we thought it was skippable: %s Actor: %s"), *GetFullNameSafe(Obj), *GetFullNameSafe(Actor));

	return bWroteSomething;
}

const TCHAR* UActorChannel::ToString(UActorChannel::ESubObjectDeleteFlag DeleteFlag)
{
	switch (DeleteFlag)
	{
	case UActorChannel::ESubObjectDeleteFlag::Destroyed:	return TEXT("Destroyed");
	case UActorChannel::ESubObjectDeleteFlag::TearOff:		return TEXT("TearOff");
	case UActorChannel::ESubObjectDeleteFlag::ForceDelete:	return TEXT("ForceDelete");
	}
	return TEXT("Missing");
}

//------------------------------------------------------

static void	DebugNetGUIDs( UWorld* InWorld )
{
	UNetDriver *NetDriver = InWorld->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	UNetConnection * Connection = (NetDriver->ServerConnection ? ToRawPtr(NetDriver->ServerConnection) : (NetDriver->ClientConnections.Num() > 0 ? ToRawPtr(NetDriver->ClientConnections[0]) : NULL));
	if (!Connection)
	{
		return;
	}

	Connection->PackageMap->LogDebugInfo(*GLog);
}

FAutoConsoleCommandWithWorld DormantActorCommand(
	TEXT("net.ListNetGUIDs"),
	TEXT( "Lists NetGUIDs for actors" ),
	FConsoleCommandWithWorldDelegate::CreateStatic(DebugNetGUIDs)
	);

//------------------------------------------------------

static void	ListOpenActorChannels( UWorld* InWorld )
{
	UNetDriver *NetDriver = InWorld->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	UNetConnection * Connection = (NetDriver->ServerConnection ? ToRawPtr(NetDriver->ServerConnection) : (NetDriver->ClientConnections.Num() > 0 ? ToRawPtr(NetDriver->ClientConnections[0]) : NULL));
	if (!Connection)
	{
		return;
	}

	TMap<UClass*, int32> ClassMap;

	for (auto It = Connection->ActorChannelConstIterator(); It; ++It)
	{
		UActorChannel* Chan = It.Value();
		UClass *ThisClass = Chan->Actor->GetClass();
		while( Cast<UBlueprintGeneratedClass>(ThisClass))
		{
			ThisClass = ThisClass->GetSuperClass();
		}

		UE_LOGF(LogNet, Warning, "Chan[%d] %ls ", Chan->ChIndex, *Chan->Actor->GetFullName() );

		int32 &cnt = ClassMap.FindOrAdd(ThisClass);
		cnt++;
	}

	// Sort by the order in which categories were edited
	struct FCompareActorClassCount
	{
		FORCEINLINE bool operator()( int32 A, int32 B ) const
		{
			return A < B;
		}
	};
	ClassMap.ValueSort( FCompareActorClassCount() );

	UE_LOGF(LogNet, Warning, "-----------------------------" );

	for (auto It = ClassMap.CreateIterator(); It; ++It)
	{
		UE_LOGF(LogNet, Warning, "%4d - %ls", It.Value(), *It.Key()->GetName() );
	}
}

FAutoConsoleCommandWithWorld ListOpenActorChannelsCommand(
	TEXT("net.ListActorChannels"),
	TEXT( "Lists open actor channels" ),
	FConsoleCommandWithWorldDelegate::CreateStatic(ListOpenActorChannels)
	);

//------------------------------------------------------

static void	DeleteDormantActor( UWorld* InWorld )
{
	UNetDriver *NetDriver = InWorld->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	UNetConnection * Connection = (NetDriver->ServerConnection ? ToRawPtr(NetDriver->ServerConnection) : (NetDriver->ClientConnections.Num() > 0 ? ToRawPtr(NetDriver->ClientConnections[0]) : NULL));
	if (!Connection)
	{
		return;
	}

	for (auto It = Connection->Driver->GetNetworkObjectList().GetAllObjects().CreateConstIterator(); It; ++It)
	{
		FNetworkObjectInfo* ActorInfo = ( *It ).Get();

		if ( !ActorInfo->DormantConnections.Num() )
		{
			continue;
		}

		AActor* ThisActor = ActorInfo->Actor;

		UE_LOGF(LogNet, Warning, "Deleting actor %ls", *ThisActor->GetName());

#if ENABLE_DRAW_DEBUG
		FBox Box = ThisActor->GetComponentsBoundingBox();

		DrawDebugBox( InWorld, Box.GetCenter(), Box.GetExtent(), FQuat::Identity, FColor::Red, true, 30 );
#endif

		ThisActor->Destroy();

		break;
	}
}

FAutoConsoleCommandWithWorld DeleteDormantActorCommand(
	TEXT("net.DeleteDormantActor"),
	TEXT( "Lists open actor channels" ),
	FConsoleCommandWithWorldDelegate::CreateStatic(DeleteDormantActor)
	);

//------------------------------------------------------
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static void	FindNetGUID( const TArray<FString>& Args, UWorld* InWorld )
{
	for (FThreadSafeObjectIterator ObjIt(UNetDriver::StaticClass()); ObjIt; ++ObjIt)
	{
		UNetDriver * Driver = Cast< UNetDriver >( *ObjIt );

		if ( Driver->HasAnyFlags( RF_ClassDefaultObject | RF_ArchetypeObject ) )
		{
			continue;
		}

		if (!FNetGUIDCache::IsHistoryEnabled())
		{
			UE_LOGF(LogNet, Warning, "FindNetGUID - GuidCacheHistory is not enabled");
			return;
		}

		if (Args.Num() <= 0)
		{
			// Display all
			for (auto It = Driver->GetNetGuidCache()->History.CreateIterator(); It; ++It)
			{
				FString Str = It.Value();
				FNetworkGUID NetGUID = It.Key();
				UE_LOGF(LogNet, Warning, "<%ls> - %ls", *NetGUID.ToString(), *Str);
			}
		}
		else
		{
			uint64 GUIDValue=0;
			TTypeFromString<uint64>::FromString(GUIDValue, *Args[0]);

			uint64 NetIndex = GUIDValue >> 1;
			bool bStatic = GUIDValue & 1;

			//@todo: add a from string method in dev only
			FNetworkGUID NetGUID = FNetworkGUID::CreateFromIndex(NetIndex, bStatic);

			// Search
			FString Str = Driver->GetNetGuidCache()->History.FindRef(NetGUID);

			if (!Str.IsEmpty())
			{
				UE_LOGF(LogNet, Warning, "Found: %ls", *Str);
			}
			else
			{
				UE_LOGF(LogNet, Warning, "No matches" );
			}
		}
	}
}

FAutoConsoleCommandWithWorldAndArgs FindNetGUIDCommand(
	TEXT("net.Packagemap.FindNetGUID"),
	TEXT( "Looks up object that was assigned a given NetGUID" ),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(FindNetGUID)
	);
#endif

//------------------------------------------------------

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static void	TestObjectRefSerialize( const TArray<FString>& Args, UWorld* InWorld )
{
	if (!InWorld || Args.Num() <= 0)
		return;

	UObject* Object = StaticFindObject( UObject::StaticClass(), NULL, *Args[0], EFindObjectFlags::None );
	if (!Object)
	{
		Object = StaticLoadObject( UObject::StaticClass(), NULL, *Args[0], NULL, LOAD_NoWarn );
	}

	if (!Object)
	{
		UE_LOGF(LogNet, Warning, "Couldn't find object: %ls", *Args[0]);
		return;
	}

	UE_LOGF(LogNet, Warning, "Repping reference to: %ls", *Object->GetName());

	UNetDriver *NetDriver = InWorld->GetNetDriver();

	for (int32 i=0; i < NetDriver->ClientConnections.Num(); ++i)
	{
		if ( NetDriver->ClientConnections[i] && NetDriver->ClientConnections[i]->PackageMap )
		{
			FBitWriter TempOut( 1024 * 10, true);
			NetDriver->ClientConnections[i]->PackageMap->SerializeObject(TempOut, UObject::StaticClass(), Object, NULL);
		}
	}

	/*
	for( auto Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		APlayerController* PlayerController = *Iterator;
		PlayerController->ClientRepObjRef(Object);
	}
	*/
}

FAutoConsoleCommandWithWorldAndArgs TestObjectRefSerializeCommand(
	TEXT("net.TestObjRefSerialize"),
	TEXT( "Attempts to replicate an object reference to all clients" ),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(TestObjectRefSerialize)
	);
#endif

