// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataDrivenCVars/DataDrivenCVars.h"
#include "HAL/ConsoleManager.h"
#include "Engine/Engine.h"
#include "Subsystems/SubsystemCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataDrivenCVars)

FDataDrivenConsoleVariable::~FDataDrivenConsoleVariable()
{
	if (!Name.IsEmpty())
	{
		UnRegister();
	}
}

void FDataDrivenConsoleVariable::Register()
{
	if (!Name.IsEmpty() && !bIsRegistered)
	{
		IConsoleVariable* CVarToAdd = IConsoleManager::Get().FindConsoleVariable(*Name);
		if (CVarToAdd == nullptr)
		{
			if (Type == FDataDrivenCVarType::CVarInt)
			{
				CVarToAdd = IConsoleManager::Get().RegisterConsoleVariable(*Name, DefaultValueInt, *ToolTip, ECVF_Default | ECVF_Scalability);
			}
			else if (Type == FDataDrivenCVarType::CVarBool)
			{
				CVarToAdd = IConsoleManager::Get().RegisterConsoleVariable(*Name, DefaultValueBool, *ToolTip, ECVF_Default | ECVF_Scalability);
			}
			else
			{
				CVarToAdd = IConsoleManager::Get().RegisterConsoleVariable(*Name, DefaultValueFloat, *ToolTip, ECVF_Default | ECVF_Scalability);
			}
		}
		CVarToAdd->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(UDataDrivenConsoleVariableSettings::OnDataDrivenChange));
		ShadowName = Name;
		ShadowToolTip = ToolTip;
		ShadowType = Type;
		bIsRegistered = true;
	}
}

void FDataDrivenConsoleVariable::UnRegister(bool bUseShadowName)
{
	const FString& NameToUnregister = bUseShadowName ? ShadowName : Name;
	if (!NameToUnregister.IsEmpty() && bIsRegistered)
	{
		IConsoleVariable* CVarToRemove = IConsoleManager::Get().FindConsoleVariable(*NameToUnregister);
		if (CVarToRemove)
		{
			FConsoleVariableDelegate NullCallback;
			CVarToRemove->SetOnChangedCallback(NullCallback);
			IConsoleManager::Get().UnregisterConsoleObject(CVarToRemove, false);
		}

		bIsRegistered = false;
	}
}

#if WITH_EDITOR
void FDataDrivenConsoleVariable::Refresh()
{
	if (ShadowName != Name)
	{
		// unregister old cvar name
		if (!ShadowName.IsEmpty())
		{
			UnRegister(true);
		}
		ShadowName = Name;
	}
	else if (ShadowToolTip != ToolTip)
	{
		UnRegister(true);
		ShadowToolTip = ToolTip;
	}
	else if (ShadowType != Type)
	{
		UnRegister(true);
		ShadowType = Type;
	}

	// make sure the cvar is registered
	Register();

	//Ensure the default value is applied, assuming no other external changes to the CVar.
	IConsoleVariable* CVarToRefresh = IConsoleManager::Get().FindConsoleVariable(*Name);
	if (CVarToRefresh)
	{
		switch (Type)
		{
		case FDataDrivenCVarType::CVarBool: CVarToRefresh->Set(DefaultValueBool, ECVF_SetByConstructor); break;
		case FDataDrivenCVarType::CVarInt: CVarToRefresh->Set(DefaultValueInt, ECVF_SetByConstructor); break;
		case FDataDrivenCVarType::CVarFloat: CVarToRefresh->Set(DefaultValueFloat, ECVF_SetByConstructor); break;
		}
	}
}
#endif

void UDataDrivenConsoleVariableSettings::PostInitProperties()
{
	Super::PostInitProperties();

	if (GetDefault<UDataDrivenConsoleVariableSettings>() != this)
	{
		// We are a temp copy so there's no need to re-register our cvars.
		// i.e. FSettingsEditorCustomization::CacheResetDefaultPropertyValues makes a temp copy to check defaults.
		return;
	}


	for (FDataDrivenConsoleVariable& CVar : CVarsArray)
	{
		CVar.Register();
	}
}

void UDataDrivenConsoleVariableSettings::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

	// Re-register all CVars after a config reload (e.g. from a plugin INI dynamic layer).
	// PostInitProperties only runs once at construction, so new entries added via plugin INIs
	// would otherwise never have Register() called on them.
	// Register() is safe to call on already-registered CVars since it skips creation if the CVar exists.
	for (FDataDrivenConsoleVariable& CVar : CVarsArray)
	{
		CVar.Register();
	}
}

void UDataDrivenConsoleVariableSettings::OnDataDrivenChange(IConsoleVariable* CVar)
{
	if (GEngine != nullptr)
	{
		UDataDrivenCVarEngineSubsystem* Subsystem = GEngine->GetEngineSubsystem<UDataDrivenCVarEngineSubsystem>();
		if (Subsystem)
		{
			const IConsoleManager& ConsoleManager = IConsoleManager::Get();
			Subsystem->OnDataDrivenCVarDelegate.Broadcast(ConsoleManager.FindConsoleObjectName(CVar));
		}
	}
}

#if WITH_EDITOR
void UDataDrivenConsoleVariableSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	for (FDataDrivenConsoleVariable& CVar : CVarsArray)
	{
		CVar.Refresh();
	}
}
#endif // #if WITH_EDITOR

FName UDataDrivenConsoleVariableSettings::GetCategoryName() const
{
	return FName("Engine");
}

