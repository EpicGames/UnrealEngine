// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DataChannel.cpp: Unreal datachannel implementation.
=============================================================================*/

#include "Net/DataReplication.h"
#include "AutoRTFM.h"
#include "Containers/StaticBitArray.h"
#include "EngineStats.h"
#include "Engine/World.h"
#include "Misc/MemStack.h"
#include "Misc/ScopeExit.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "Net/Core/Misc/NetContext.h"
#include "Net/NetworkProfiler.h"
#include "Engine/PackageMapClient.h"
#include "Net/RepLayout.h"
#include "Engine/ActorChannel.h"
#include "Net/Serialization/FastArraySerializer.h"
#include "Engine/Engine.h"
#include "Engine/NetConnection.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/NetCoreModule.h"
#include "HAL/LowLevelMemStats.h"
#include "Net/Core/PushModel/Types/PushModelPerNetDriverState.h"
#include "Net/Core/Connection/RPCDoSDetection.h"
#include "UObject/CoreNetContext.h"
#include "UObject/RemoteExecutor.h"

DECLARE_LLM_MEMORY_STAT(TEXT("NetObjReplicator"), STAT_NetObjReplicatorLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(NetObjReplicator, NAME_None, TEXT("Networking"), GET_STATFNAME(STAT_NetObjReplicatorLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));

DECLARE_CYCLE_STAT(TEXT("Custom Delta Property Rep Time"), STAT_NetReplicateCustomDeltaPropTime, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("ReceiveRPC"), STAT_NetReceiveRPC, STATGROUP_Game);

static TAutoConsoleVariable<int32> CVarMaxRPCPerNetUpdate(
	TEXT("net.MaxRPCPerNetUpdate"),
	2,
	TEXT("Maximum number of unreliable multicast RPC calls allowed per net update, additional ones will be dropped"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarDelayUnmappedRPCs(
	TEXT("net.DelayUnmappedRPCs"),
	0,
	TEXT("If true delay received RPCs with unmapped object references until they are received or loaded, ")
	TEXT("if false RPCs will execute immediately with null parameters. ")
	TEXT("This can be used with net.AllowAsyncLoading to avoid null asset parameters during async loads."),
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarNetReplicationDebugProperty(
	TEXT("net.Replication.DebugProperty"),
	TEXT(""),
	TEXT("Debugs Replication of property by name, ")
	TEXT("this should be set to the partial name of the property to debug"),
	ECVF_Default);

int32 GNetRPCDebug = 0;
static FAutoConsoleVariableRef CVarNetRPCDebug(
	TEXT("net.RPC.Debug"),
	GNetRPCDebug,
	TEXT("Print all RPC bunches sent over the network\n")
	TEXT(" 0: no print.\n")
	TEXT(" 1: Print bunches as they are sent."),
	ECVF_Default);

int32 GSupportsFastArrayDelta = 1;
static FAutoConsoleVariableRef CVarSupportsFastArrayDelta(
	TEXT("net.SupportFastArrayDelta"),
	GSupportsFastArrayDelta,
	TEXT("Whether or not Fast Array Struct Delta Serialization is enabled.")
);

bool GbPushModelSkipUndirtiedReplicators = false;
FAutoConsoleVariableRef CVarPushModelSkipUndirtiedReplicators(
	TEXT("net.PushModelSkipUndirtiedReplication"),
	GbPushModelSkipUndirtiedReplicators,
	TEXT("When true, skip replicating any objects that we can safely see aren't dirty."));

bool GbPushModelSkipUndirtiedFastArrays = false;
FAutoConsoleVariableRef CVarPushModelSkipUndirtiedFastArrays(
	TEXT("net.PushModelSkipUndirtiedFastArrays"),
	GbPushModelSkipUndirtiedFastArrays,
	TEXT("When true, include fast arrays when skipping objects that we can safely see aren't dirty."));

static TAutoConsoleVariable<int32> CVarQueueRpcsForTransactions(
	TEXT("net.QueueRpcsForTransactions"),
	1,
	TEXT("When set to 1, any transactional connection will queue RPCs and execute them in transactions allowing for rescheduling"));

static TAutoConsoleVariable<bool> CVarEnqueueTransactionalRpcs(
	TEXT("net.EnqueueTransactionalRpcs"),
	true,
	TEXT("When set to true (and net.QueueRpcsForTransactions to 1), each individual RPC will be enqueued rather than immediately executed (allows further unblocking of transactions)"));

extern int32 GNumSkippedObjectEmptyUpdates;

class FNetSerializeCB : public INetSerializeCB
{
private:

	// This is an acceleration so if we make back to back requests for the same type
	// we don't have to do repeated lookups.
	struct FCachedRequestState
	{
		UClass* ObjectClass = nullptr;
		UScriptStruct* Struct = nullptr;
		TSharedPtr<FRepLayout> RepLayout;
		bool bWasRequestFromClass = false;
	};

public:

	FNetSerializeCB():
		Driver(nullptr)
	{
		check(0);
	}

	FNetSerializeCB(UNetDriver* InNetDriver):
		Driver(InNetDriver)
	{
	}

	void SetChangelistMgr(TSharedPtr<FReplicationChangelistMgr> InChangelistMgr)
	{
		ChangelistMgr = InChangelistMgr;
	}

private:

	void UpdateCachedRepLayout()
	{
		if (!CachedRequestState.RepLayout.IsValid())
		{
			if (CachedRequestState.bWasRequestFromClass)
			{
				CachedRequestState.RepLayout = Driver->GetObjectClassRepLayout(CachedRequestState.ObjectClass);
			}
			else
			{
				CachedRequestState.RepLayout = Driver->GetStructRepLayout(CachedRequestState.Struct);
			}
		}
	}

	void UpdateCachedState(UClass* ObjectClass, UStruct* Struct)
	{
		if (CachedRequestState.ObjectClass != ObjectClass)
		{
			CachedRequestState.ObjectClass = ObjectClass;
			CachedRequestState.Struct = CastChecked<UScriptStruct>(Struct);
			CachedRequestState.bWasRequestFromClass = true;
			CachedRequestState.RepLayout.Reset();
		}
	}

	void UpdateCachedState(UStruct* Struct)
	{
		if (CachedRequestState.Struct != Struct || CachedRequestState.ObjectClass != nullptr)
		{
			CachedRequestState.ObjectClass = nullptr;
			CachedRequestState.Struct = CastChecked<UScriptStruct>(Struct);
			CachedRequestState.bWasRequestFromClass = false;
			CachedRequestState.RepLayout.Reset();
		}
	}

public:

	virtual void NetSerializeStruct(FNetDeltaSerializeInfo& Params) override final
	{
		UpdateCachedState(Params.Struct);
		FBitArchive& Ar = Params.Reader ? static_cast<FBitArchive&>(*Params.Reader) : static_cast<FBitArchive&>(*Params.Writer);
		Params.bOutHasMoreUnmapped = false;

		if (EnumHasAnyFlags(CachedRequestState.Struct->StructFlags, STRUCT_NetSerializeNative))
		{
			UScriptStruct::ICppStructOps* CppStructOps = CachedRequestState.Struct->GetCppStructOps();
			check(CppStructOps);
			bool bSuccess = true;

			if (!CppStructOps->NetSerialize(Ar, Params.Map, bSuccess, Params.Data))
			{
				Params.bOutHasMoreUnmapped = true;
			}

			if (!bSuccess)
			{
				UE_LOGF(LogRep, Warning, "NetSerializeStruct: Native NetSerialize %ls failed.", *Params.Struct->GetFullName());
			}
		}
		else
		{
			UpdateCachedRepLayout();
			UPackageMapClient* PackageMapClient = ((UPackageMapClient*)Params.Map);

			if (PackageMapClient && PackageMapClient->GetConnection()->IsInternalAck())
			{
				if (Ar.IsSaving())
				{
					TArray< uint16 > Changed;
					CachedRequestState.RepLayout->SendProperties_BackwardsCompatible(nullptr, nullptr, (uint8*)Params.Data, PackageMapClient->GetConnection(), static_cast<FNetBitWriter&>(Ar), Changed);
				}
				else
				{
					bool bHasGuidsChanged = false;
					CachedRequestState.RepLayout->ReceiveProperties_BackwardsCompatible(PackageMapClient->GetConnection(), nullptr, Params.Data, static_cast<FNetBitReader&>(Ar), Params.bOutHasMoreUnmapped, false, bHasGuidsChanged, Params.Object);
				}
			}
			else
			{
				CachedRequestState.RepLayout->SerializePropertiesForStruct(Params.Struct, Ar, Params.Map, Params.Data, Params.bOutHasMoreUnmapped, Params.Object);
			}
		}
	}

	virtual bool NetDeltaSerializeForFastArray(FFastArrayDeltaSerializeParams& Params) override final
	{
		UpdateCachedState(Params.DeltaSerializeInfo.Object->GetClass(), Params.DeltaSerializeInfo.Struct);
		UpdateCachedRepLayout();

		const ERepLayoutResult Result = CachedRequestState.RepLayout->DeltaSerializeFastArrayProperty(Params, ChangelistMgr.Get());

		if (ERepLayoutResult::FatalError == Result && Params.DeltaSerializeInfo.Connection)
		{
			Params.DeltaSerializeInfo.Connection->SetPendingCloseDueToReplicationFailure();
		}

		return ERepLayoutResult::Success == Result;
	}

	virtual void GatherGuidReferencesForFastArray(FFastArrayDeltaSerializeParams& Params) override final
	{
		UpdateCachedState(Params.DeltaSerializeInfo.Object->GetClass(), Params.DeltaSerializeInfo.Struct);
		UpdateCachedRepLayout();
		CachedRequestState.RepLayout->GatherGuidReferencesForFastArray(Params);
	}

	virtual bool MoveGuidToUnmappedForFastArray(FFastArrayDeltaSerializeParams& Params) override final
	{
		UpdateCachedState(Params.DeltaSerializeInfo.Object->GetClass(), Params.DeltaSerializeInfo.Struct);
		UpdateCachedRepLayout();
		return CachedRequestState.RepLayout->MoveMappedObjectToUnmappedForFastArray(Params);
	}

	virtual void UpdateUnmappedGuidsForFastArray(FFastArrayDeltaSerializeParams& Params) override final
	{
		UpdateCachedState(Params.DeltaSerializeInfo.Object->GetClass(), Params.DeltaSerializeInfo.Struct);
		UpdateCachedRepLayout();
		CachedRequestState.RepLayout->UpdateUnmappedGuidsForFastArray(Params);
	}

public:

	static bool SendCustomDeltaProperty(
		const FRepLayout& RepLayout,
		FNetDeltaSerializeInfo& Params,
		uint16 CustomDeltaIndex)
	{
		return RepLayout.SendCustomDeltaProperty(Params, CustomDeltaIndex);
	}

	static bool ReceiveCustomDeltaProperty(
		const FRepLayout& RepLayout,
		FReceivingRepState* ReceivingRepState,
		FNetDeltaSerializeInfo& Params,
		FStructProperty* ReplicatedProp)
	{
		return RepLayout.ReceiveCustomDeltaProperty(ReceivingRepState, Params, ReplicatedProp);
	}

	static void PreSendCustomDeltaProperties(
		const FRepLayout& RepLayout,
		UObject* Object,
		UNetConnection* Connection,
		FReplicationChangelistMgr& ChangelistMgr,
		TArray<TSharedPtr<INetDeltaBaseState>>& CustomDeltaStates)
	{
		RepLayout.PreSendCustomDeltaProperties(Object, Connection, ChangelistMgr, Connection->Driver->ReplicationFrame, CustomDeltaStates);
	}

	static void PostSendCustomDeltaProperties(
		const FRepLayout& RepLayout,
		UObject* Object,
		UNetConnection* Connection,
		FReplicationChangelistMgr& ChangelistMgr,
		TArray<TSharedPtr<INetDeltaBaseState>>& CustomDeltaStates)
	{
		RepLayout.PostSendCustomDeltaProperties(Object, Connection, ChangelistMgr, CustomDeltaStates);
	}

	static uint16 GetNumLifetimeCustomDeltaProperties(const FRepLayout& RepLayout)
	{
		return RepLayout.GetNumLifetimeCustomDeltaProperties();
	}

	static uint16 GetLifetimeCustomDeltaPropertyRepIndex(const FRepLayout& RepLayout, const uint16 CustomDeltaPropertyIndex)
	{
		return RepLayout.GetLifetimeCustomDeltaPropertyRepIndex(CustomDeltaPropertyIndex);
	}

	static FProperty* GetLifetimeCustomDeltaProperty(const FRepLayout& RepLayout, const uint16 CustomDeltaPropertyIndex)
	{
		return RepLayout.GetLifetimeCustomDeltaProperty(CustomDeltaPropertyIndex);
	}

	static ERepLayoutResult UpdateChangelistMgr(
		const FRepLayout& RepLayout,
		FSendingRepState* RESTRICT RepState,
		FReplicationChangelistMgr& InChangelistMgr,
		const UObject* InObject,
		const uint32 ReplicationFrame,
		const FReplicationFlags& RepFlags,
		const bool bForceCompare)
	{
		return RepLayout.UpdateChangelistMgr(RepState, InChangelistMgr, InObject, ReplicationFrame, RepFlags, bForceCompare);
	}

	static const ELifetimeCondition GetLifetimeCustomDeltaPropertyCondition(const FRepLayout& RepLayout, const uint16 CustomDeltaPropertyIndex)
	{
		return RepLayout.GetLifetimeCustomDeltaPropertyCondition(CustomDeltaPropertyIndex);
	}

private:

	UNetDriver* Driver;
	FCachedRequestState CachedRequestState;
	TSharedPtr<FReplicationChangelistMgr> ChangelistMgr;
};

FObjectReplicator::FObjectReplicator()
	: bLastUpdateEmpty(false)
	, bOpenAckCalled(false)
	, bForceUpdateUnmapped(false)
	, bHasReplicatedProperties(false)
	, bSentSubObjectCreation(false)
	, bSupportsFastArrayDelta(false)
	, bCanUseNonDirtyOptimization(false)
	, bDirtyForReplay(true)
	, ObjectClass(nullptr)
	, ObjectPtr(nullptr)
	, Connection(nullptr)
	, OwningChannel(nullptr)
	, RemoteFunctions(nullptr)
{
}

FObjectReplicator::~FObjectReplicator()
{
	CleanUp();
}

bool FObjectReplicator::SendCustomDeltaProperty(UObject* InObject, uint16 CustomDeltaIndex, FNetBitWriter& OutBunch, TSharedPtr<INetDeltaBaseState>& NewFullState, TSharedPtr<INetDeltaBaseState>& OldState)
{
	check(!NewFullState.IsValid()); // NewState is passed in as nullptr and instantiated within this function if necessary
	check(RepLayout);

	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_NetSerializeItemDeltaTime, GUseDetailedScopeCounters);

	UNetDriver* const ConnectionDriver = Connection->GetDriver();
	FNetSerializeCB NetSerializeCB(ConnectionDriver);
	NetSerializeCB.SetChangelistMgr(ChangelistMgr);

	FNetDeltaSerializeInfo Parms;
	Parms.Object = InObject;
	Parms.CustomDeltaObject = GetObject();
	Parms.Writer = &OutBunch;
	Parms.Map = Connection->PackageMap;
	Parms.OldState = OldState.Get();
	Parms.NewState = &NewFullState;
	Parms.NetSerializeCB = &NetSerializeCB;
	Parms.bIsWritingOnClient = ConnectionDriver && ConnectionDriver->GetWorld() && ConnectionDriver->GetWorld()->IsRecordingClientReplay();
	Parms.CustomDeltaIndex = CustomDeltaIndex;
	Parms.bSupportsFastArrayDeltaStructSerialization = bSupportsFastArrayDelta;
	Parms.Connection = Connection;
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	if (ConnectionDriver && ConnectionDriver->GetWorld())
	{
		Parms.bCallFastArrayClientCallbacks = ConnectionDriver->GetWorld()->IsActorTickAndUserCallbacksEnabled();
	}
#endif
	Parms.bInternalAck = Connection->IsInternalAck();

	// When initializing baselines we should not modify the source data if it originates from the CDO or archetype
	Parms.bIsInitializingBaseFromDefault = Parms.Object && Parms.Object->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject);

	return FNetSerializeCB::SendCustomDeltaProperty(*RepLayout, Parms, CustomDeltaIndex);
}

/** 
 *	Utility function to make a copy of the net properties 
 *	@param	Source - Memory to copy initial state from
**/
void FObjectReplicator::InitRecentProperties(uint8* Source)
{
	// TODO: Could we just use the cached ObjectPtr here?
	UObject* MyObject = GetObject();

	check(MyObject);
	check(Connection);
	check(!RepState.IsValid());

	UNetDriver* const ConnectionDriver = Connection->GetDriver();
	const bool bIsServer = ConnectionDriver->IsServer();
	const bool bCreateSendingState = bIsServer || ConnectionDriver->MaySendProperties();
	const FRepLayout& LocalRepLayout = *RepLayout;

	UClass* InObjectClass = MyObject->GetClass();

	// Initialize the RepState memory
	// Clients don't need RepChangedPropertyTracker's, as they are mainly
	// used temporarily disable property replication, or store data
	// for replays (and the DemoNetDriver will be acts as a server during recording).
	TSharedPtr<FRepChangedPropertyTracker> RepChangedPropertyTracker = bCreateSendingState ? ConnectionDriver->FindOrCreateRepChangedPropertyTracker(MyObject) : nullptr;

	// If acting as a server and are IsInternalAck, that means we're recording.
	// In that case, we don't need to create any receiving state, as no one will be sending data to us.
	ECreateRepStateFlags Flags = (Connection->IsInternalAck() && bIsServer) ? ECreateRepStateFlags::SkipCreateReceivingState : ECreateRepStateFlags::None;

	UE_AUTORTFM_OPEN
	{
		RepState = LocalRepLayout.CreateRepState(Source, RepChangedPropertyTracker, Flags);
	};

	// RepState not valid at the start of this function so just go back to being a nullptr, and let the memory be cleaned up 
	UE_AUTORTFM_ONABORT(this)
	{
		RepState = nullptr;
	};

	if (!bCreateSendingState)
	{
		// Clients don't need to initialize shadow state (and in fact it causes issues in replays)
		return;
	}

	bSupportsFastArrayDelta = !!GSupportsFastArrayDelta;

	if (FSendingRepState* SendingRepState = RepState->GetSendingRepState())
	{
		// We should just update this method to accept an object pointer.
		UObject* UseObject = reinterpret_cast<UObject*>(Source);

		// Init custom delta property state
		const uint16 NumLifetimeCustomDeltaProperties = FNetSerializeCB::GetNumLifetimeCustomDeltaProperties(LocalRepLayout);
		SendingRepState->RecentCustomDeltaState.SetNum(NumLifetimeCustomDeltaProperties);

		const bool bIsRecordingReplay = Connection->IsInternalAck();

		if (bIsRecordingReplay)
		{
			SendingRepState->CDOCustomDeltaState.SetNum(NumLifetimeCustomDeltaProperties);
			SendingRepState->CheckpointCustomDeltaState.SetNum(NumLifetimeCustomDeltaProperties);
		}

		for (uint16 CustomDeltaProperty = 0; CustomDeltaProperty < NumLifetimeCustomDeltaProperties; ++CustomDeltaProperty)
		{
			FOutBunch DeltaState(Connection->PackageMap);
			UE::Net::FNetTokenExportScope NetTokenExportScope(DeltaState, ConnectionDriver->GetNetTokenStore(), DeltaState.NetTokensPendingExport, "SendPropertiesForRPC");

			TSharedPtr<INetDeltaBaseState>& NewState = SendingRepState->RecentCustomDeltaState[CustomDeltaProperty];
			NewState.Reset();

			TSharedPtr<INetDeltaBaseState> OldState;

			SendCustomDeltaProperty(UseObject, CustomDeltaProperty, DeltaState, NewState, OldState);

			if (bIsRecordingReplay)
			{
				// Store the initial delta state in case we need it for when we're asked to resend all data since channel was first opened (bResendAllDataSinceOpen)
				SendingRepState->CDOCustomDeltaState[CustomDeltaProperty] = NewState;
				SendingRepState->CheckpointCustomDeltaState[CustomDeltaProperty] = NewState;
			}
		}
	}
}

/** Takes Data, and compares against shadow state to log differences */
bool FObjectReplicator::ValidateAgainstState( const UObject* ObjectState )
{
	if (!RepLayout.IsValid())
	{
		UE_LOGF(LogRep, Warning, "ValidateAgainstState: RepLayout.IsValid() == false");
		return false;
	}

	if (!RepState.IsValid())
	{
		UE_LOGF(LogRep, Warning, "ValidateAgainstState: RepState.IsValid() == false");
		return false;
	}

	if (!ChangelistMgr.IsValid())
	{
		UE_LOGF(LogRep, Warning, "ValidateAgainstState: ChangelistMgr.IsValid() == false");
		return false;
	}

	FRepChangelistState* ChangelistState = ChangelistMgr->GetRepChangelistState();
	if (ChangelistState == nullptr)
	{
		UE_LOGF(LogRep, Warning, "ValidateAgainstState: ChangelistState == nullptr");
		return false;
	}

	FRepShadowDataBuffer ShadowData(ChangelistState->StaticBuffer.GetData());
	FConstRepObjectDataBuffer ObjectData(ObjectState);

	if (RepLayout->DiffProperties(nullptr, ShadowData, ObjectData, EDiffPropertiesFlags::None))
	{
		UE_LOGF(LogRep, Warning, "ValidateAgainstState: Properties changed for %ls", *ObjectState->GetName());
		return false;
	}

	return true;
}

void FObjectReplicator::InitWithObject(UObject* InObject, UNetConnection* InConnection, bool bUseDefaultState)
{
	LLM_SCOPE_BYTAG(NetObjReplicator);

	check(GetObject() == nullptr);
	check(ObjectClass == nullptr);
	check(bLastUpdateEmpty == false);
	check(Connection == nullptr);
	check(OwningChannel == nullptr);
	check(!RepState.IsValid());
	check(RemoteFunctions == nullptr);
	check(!RepLayout.IsValid());

	SetObject(InObject);

	if (!WeakObjectPtr.IsValid())
	{
		// This may seem weird that we're checking for nullptr, but the SetObject above will wrap this object with TWeakObjectPtr
		// If the object is pending kill, it will switch to nullptr, we're just making sure we handle this invalid edge case
		UE_LOGF(LogRep, Error, "InitWithObject: Invalid Object pointer %p with name `%ls`", InObject, ToCStr(GetNameSafe(InObject)));
		return;
	}

	ObjectClass = InObject->GetClass();
	Connection = InConnection;
	RemoteFunctions = nullptr;
	bHasReplicatedProperties = false;
	bOpenAckCalled = false;
	RepState = nullptr;
	OwningChannel = nullptr;		// Initially nullptr until StartReplicating is called
	TrackedGuidMemoryBytes = 0;

	RepLayout = Connection->Driver->GetObjectClassRepLayout(ObjectClass);

	// Make a copy of the net properties
	uint8* Source = bUseDefaultState ? (uint8*)InObject->GetArchetype() : (uint8*)InObject;

	if ((Source == nullptr) && bUseDefaultState)
	{
		if (ObjectClass != nullptr)
		{
			UE_LOGF(LogRep, Error, "FObjectReplicator::InitWithObject: Invalid object archetype, initializing shadow state to class default state: %ls", *GetFullNameSafe(InObject));
			Source = (uint8*)ObjectClass->GetDefaultObject();
		}
		else
		{
			UE_LOGF(LogRep, Error, "FObjectReplicator::InitWithObject: Invalid object archetype and class, initializing shadow state to current object state: %ls", *GetFullNameSafe(InObject));
			Source = (uint8*)InObject;
		}
	}

	InitRecentProperties(Source);

	{
		LLM_SCOPE_BYTAG(NetDriver);
		Connection->Driver->AllOwnedReplicators.Add(this);
	}

	if (GbPushModelSkipUndirtiedFastArrays)
	{
		bCanUseNonDirtyOptimization =
			GbPushModelSkipUndirtiedReplicators &&
			(RepLayout->IsEmpty() || EnumHasAnyFlags(RepLayout->GetFlags(), ERepLayoutFlags::FullPushSupport));
	}
	else
	{
		bCanUseNonDirtyOptimization =
			GbPushModelSkipUndirtiedReplicators &&
			(RepLayout->IsEmpty() || EnumHasAnyFlags(RepLayout->GetFlags(), ERepLayoutFlags::FullPushProperties)) &&
			RepState->GetSendingRepState() &&
			RepState->GetSendingRepState()->RecentCustomDeltaState.Num() == 0;
	}
}

void FObjectReplicator::CleanUp()
{
	if ( OwningChannel != nullptr )
	{
		StopReplicating( OwningChannel );		// We shouldn't get here, but just in case
	}

	if ( Connection != nullptr )
	{
		for ( const FNetworkGUID& GUID : ReferencedGuids )
		{
			TSet< FObjectReplicator* >& Replicators = Connection->Driver->GuidToReplicatorMap.FindChecked( GUID );

			Replicators.Remove( this );

			if ( Replicators.Num() == 0 )
			{
				Connection->Driver->GuidToReplicatorMap.Remove( GUID );
			}
		}

		Connection->Driver->UnmappedReplicators.Remove( this );
		Connection->Driver->ReplicatorsWithPendingTransactions.Remove( this );

		Connection->Driver->TotalTrackedGuidMemoryBytes -= TrackedGuidMemoryBytes;

		Connection->Driver->AllOwnedReplicators.Remove( this );
	}
	else
	{
		ensureMsgf( TrackedGuidMemoryBytes == 0, TEXT( "TrackedGuidMemoryBytes should be 0" ) );
		ensureMsgf( ReferencedGuids.Num() == 0, TEXT( "ReferencedGuids should be 0" ) );
	}

	ReferencedGuids.Empty();
	TrackedGuidMemoryBytes = 0;

	SetObject( nullptr );

	ObjectClass					= nullptr;
	Connection					= nullptr;
	RemoteFunctions				= nullptr;
	bHasReplicatedProperties	= false;
	bSentSubObjectCreation		= false;
	bOpenAckCalled				= false;

	RepState = nullptr;
	CheckpointRepState = nullptr;
}

void FObjectReplicator::StartReplicating(class UActorChannel * InActorChannel)
{
	check(OwningChannel == nullptr);
	check(InActorChannel != nullptr);
	check(InActorChannel->Connection == Connection);

	UObject* const Object = GetObject();
	if (Object == nullptr)
	{
		UE_LOGF(LogRep, Error, "StartReplicating: Object == nullptr");
		return;
	}

	if (!ensureMsgf(ObjectClass != nullptr, TEXT( "StartReplicating: ObjectClass == nullptr. Object = %s. Channel actor = %s. %s" ), *GetFullNameSafe(Object), *GetFullNameSafe(InActorChannel->GetActor()), *InActorChannel->Connection->Describe()))
	{
		return;
	}

	if (UClass* const ObjectPtrClass = Object->GetClass())
	{
		// Something is overwriting a bit in the ObjectClass pointer so it's becoming invalid - fix up the pointer to prevent crashing later until the real cause can be identified.
		if (!ensureMsgf(ObjectClass == ObjectPtrClass, TEXT("StartReplicating: ObjectClass and ObjectPtr's class are not equal and they should be. Object = %s. Channel actor = %s. %s"), *GetFullNameSafe(Object), *GetFullNameSafe(InActorChannel->GetActor()), *InActorChannel->Connection->Describe()))
		{
			ObjectClass = ObjectPtrClass;
		}
	}

	OwningChannel = InActorChannel;

	UNetDriver* const ConnectionNetDriver = Connection->GetDriver();

	// Cache off netGUID so if this object gets deleted we can close it
	ObjectNetGUID = ConnectionNetDriver->GetNetGuidCache()->GetOrAssignNetGUID( Object );

	const bool bIsValidToReplicate = !ObjectNetGUID.IsDefault() && ObjectNetGUID.IsValid();
	if (UNLIKELY(!bIsValidToReplicate))
	{
		// This has mostly been seen when doing a Seamless Travel Restart.
		// In that case, the server can think the client is still on the same map and replicate objects before
		// the client has finished the travel, and the client will later remove those references from the Package Map.
		UE_LOGF(LogRep, Error, "StartReplicating: Invalid Net GUID. Object may fail to replicate properties or handle RPCs. Object %ls", *Object->GetPathName());
		return;
	}


	bOpenAckCalled = false;

	if (ConnectionNetDriver->IsServer() || ConnectionNetDriver->MaySendProperties())
	{
		// We don't need to handle retirement if our connection is reliable.
		if (!Connection->IsInternalAck())
		{
			if (FSendingRepState * SendingRepState = RepState.IsValid() ? RepState->GetSendingRepState() : nullptr)
			{
				// Allocate retirement list.
				// SetNum now constructs, so this is safe

				check(RepLayout);
				const FRepLayout& LocalRepLayout = *RepLayout;
				const int32 NumLifetimeCustomDeltaProperties = FNetSerializeCB::GetNumLifetimeCustomDeltaProperties(LocalRepLayout);

				SendingRepState->Retirement.SetNum(NumLifetimeCustomDeltaProperties);

				if (OwningChannel->SpawnAcked)
				{
					// Since SpawnAcked is already true, this should be true as well - this helps prevent an issue where actors can't go dormant if this object replicator is created after the actor's spawn has
					// been acked by the client
					SendingRepState->bOpenAckedCalled = true;
					UE_LOGF(LogRep, Verbose, "FObjectReplicator::InitRecentProperties -  OwningChannel->SpawnAcked is true, also setting SendingRepState->bOpenAckedCalled to true. Object = %ls. Channel actor = %ls.",
						*GetFullNameSafe(Object), *GetFullNameSafe(InActorChannel->GetActor()));
				}
			}
		}

		const UWorld* const World = ConnectionNetDriver->GetWorld();
		UNetDriver* const WorldNetDriver = World ? World->GetNetDriver() : nullptr;

		// Prefer the changelist manager on the main net driver (so we share across net drivers if possible)
		if (WorldNetDriver && WorldNetDriver->IsServer())
		{
			ChangelistMgr = WorldNetDriver->GetReplicationChangeListMgr(Object);
		}
		else
		{
			ChangelistMgr = ConnectionNetDriver->GetReplicationChangeListMgr(Object);
		}
	}
}

void ValidateRetirementHistory(const FPropertyRetirement & Retire, const UObject* Object)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FPropertyRetirement * Rec = Retire.Next;	// Note the first element is 'head' that we don't actually use

	FPacketIdRange LastRange;

	while ( Rec != nullptr )
	{
		checkf( Rec->OutPacketIdRange.Last >= Rec->OutPacketIdRange.First, TEXT( "Invalid packet id range (Last < First). Object: %s" ), *GetFullNameSafe(Object) );
		checkf( Rec->OutPacketIdRange.First >= LastRange.Last, TEXT( "Invalid packet id range (First < LastRange.Last). Object: %s" ), *GetFullNameSafe(Object) );		// Bunch merging and queuing can cause this overlap

		LastRange = Rec->OutPacketIdRange;

		Rec = Rec->Next;
	}
#endif
}

void FObjectReplicator::StopReplicating( class UActorChannel * InActorChannel )
{
	check(OwningChannel != nullptr);
	check(OwningChannel->Connection == Connection);
	check(OwningChannel == InActorChannel);

	OwningChannel = nullptr;

	const UObject* Object = GetObject();

	if (FSendingRepState* SendingRepState = RepState.IsValid() ? RepState->GetSendingRepState() : nullptr)
	{
		// Cleanup retirement records
		for (int32 i = SendingRepState->Retirement.Num() - 1; i >= 0; i--)
		{
			FPropertyRetirement& Retirement = SendingRepState->Retirement[i];
			ValidateRetirementHistory(Retirement, Object);

			FPropertyRetirement* Rec = Retirement.Next;
			Retirement.Next = nullptr;

			// We dont need to explicitly delete Retirement, but anything in the Next chain needs to be.
			while (Rec != nullptr)
			{
				FPropertyRetirement* Next = Rec->Next;
				delete Rec;
				Rec = Next;
			}
		}

		SendingRepState->Retirement.Empty();
	}

	PendingLocalRPCs.Empty();

	if (RemoteFunctions != nullptr)
	{
		delete RemoteFunctions;
		RemoteFunctions = nullptr;
	}
}

/**
 * Handling NAKs / Property Retransmission.
 *
 * Note, NACK handling only occurs on connections that "replicate" data, which is currently
 * only Servers. RPC retransmission is handled elsewhere.
 *
 * RepLayouts:
 *
 *		As we send properties through FRepLayout the is a Changelist Manager that is shared
 *		between all connections tracks sets of properties that were recently changed (history items),
 *		as well as one aggregate set of all properties that have ever been sent.
 *
 *		Each Sending Rep State, which is connection unique, also tracks the set of changed
 *		properties. These history items will only be created when replicating the object,
 *		so there will be fewer of them in general, but they will still contain any properties
 *		that compared differently (not *just* the properties that were actually replicated).
 *
 *		Whenever a NAK is received, we will iterate over the SendingRepState changelist
 *		and mark any of the properties sent in the NAKed packet for retransmission.
 *
 *		The next time Properties are replicated for the Object, we will merge in any changelists
 *		from NAKed history items.
 *
 * Custom Delta Properties:
 *
 *		For Custom Delta Properties (CDP), we rely primarily on FPropertyRetirements and INetDeltaBaseState
 *		for tracking property retransmission.
 *
 *		INetDeltaBaseStates are used to tracked internal state specific to a given type of CDP.
 *		For example, Fast Array Replicators will use FNetFastTArrayBaseState, or some type
 *		derived from that.
 *
 *		When an FObjectReplicator is created, we will create an INetDeltaBaseState for every CDP,
 *		as well as a dummy FPropertyRetirement. This Property Retirement is used as the head
 *		of a linked list of Retirements, and is generally never populated with any useful information.
 *
 *		Every time we replicate a CDP, we will pass in the most recent Base State, and we will be
 *		returned a new CDP. If data is actually sent, then we will create a new Property Retirement,
 *		adding it as the tail of our linked list. The new Property Retirement will also hold a reference
 *		to the old INetDeltaBaseState (i.e., the state of the CDP before it replicated its properties).
 *
 *		Just before replicating, we will go through and free any ACKed FPropertyRetirments (see
 *		UpdateAckedRetirements).
 *
 *		After replicating, we will cache off the returned Base State to be used as the "old" state
 *		the next time the property is replicated.
 *
 *		Whenever a NAK is received, we will run through our Property Retirements. Any retirements
 *		that predate the NACK will be removed and treated as if they were ACKs. The first
 *		retirement that is found to be within the NAKed range will have its INetDeltaBaseState
 *		restored (which should be the state before the NAKed packet was sent), and then
 *		that retirement as well as all remaining will be removed.
 *
 *		The onus is then on the CDP to resend any necessary properties based on its current / live
 *		state and the restored Net Delta Base State.
 *
 * Fast Array Properties:
 *
 *		Fast Array Properties are implemented as Custom Delta Properties (CDP). Therefore, they mostly
 *		follow the flow laid out above.

 *		FNetFastTArrayBaseState is the basis for all Fast Array Serializer INetDeltaBaseStates.
 *		This struct tracks the Replication Key of the Array, the ID to Replication Key map of individual
 *		Array Items, and a History Number.
 *
 *		As we replicate Fast Array Properties, we use the Array Replication key to see if anything
 *		is possibly dirty in the Array and the ID to Replication map to see which Array Element
 *		items actually are dirty. A mismatch between the Net Base State Key and the Key stored on
 *		the live Fast Array (either the Array Replication Key, or any Item Key) is how we determine
 *		if the Array or Items is dirty.
 *
 *		Whenever a NAK is received, our Old Base State will be reset to the last known ACKed value,
 *		as described in the CDP section above. This means that our Array Replication Key and ID To
 *		Item Replication Key should be reset to those states, forcing a mismatch the next time we
 *		replicate if anything has changed.
 *
 *		When net.SupportFastArrayDelta is enabled, we perform an additional step in which we actually
 *		compare the properties of dirty items. This is very similar to normal Property replication
 *		using RepLayouts, and leverages most of the same code.
 *
 *		This includes tracking history items just like Rep Layout. Instead of tracking histories per
 *		Sending Rep State / Per Connection, we just manage a single set of Histories on the Rep
 *		Changelist Mgr. Changelists are stored per Fast Array Item, and are referenced via ID.
 *
 *		Whenever we go to replicate a Fast Array Item, we will merge together all changelists since
 *		we last sent that item, and send those accumulated changes.
 *
 *		This means that property retransmission for Fast Array Items is an amalgamation of Rep Layout
 *		retransmission and CDP retransmission.
 *
 *		Whenever a NAK is received, our History Number should be reset to the last known  ACKed value,
 *		and that should be enough to force us to accumulate any of the NAKed item changelists.
 */

void FObjectReplicator::ReceivedNak( int32 NakPacketId )
{
	const UObject* Object = GetObject();

	if (Object == nullptr)
	{
		UE_LOGF(LogNet, Verbose, "FObjectReplicator::ReceivedNak: Object == nullptr");
		return;
	}
	else if (ObjectClass == nullptr)
	{
		UE_LOGF(LogNet, Verbose, "FObjectReplicator::ReceivedNak: ObjectClass == nullptr");
	}
	else if (!RepLayout->IsEmpty())
	{
		if (FSendingRepState* SendingRepState = RepState.IsValid() ? RepState->GetSendingRepState() : nullptr)
		{
			// This is used to determine if we can skip the update,
			// unconditionally decrementing leads to objects with no custom delta serializer
			// to be unable to be skipped for replication since their value is never updated elsewhere
			if (SendingRepState->CustomDeltaChangeIndex != 0)
			{
				SendingRepState->CustomDeltaChangeIndex--;
			}
			
			// Go over properties tracked with histories, and mark them as needing to be resent.
			for (int32 i = SendingRepState->HistoryStart; i < SendingRepState->HistoryEnd; ++i)
			{
				const int32 HistoryIndex = i % FSendingRepState::MAX_CHANGE_HISTORY;

				FRepChangedHistory& HistoryItem = SendingRepState->ChangeHistory[HistoryIndex];

				if (!HistoryItem.Resend && HistoryItem.OutPacketIdRange.InRange(NakPacketId))
				{
					check(HistoryItem.Changed.Num() > 0);
					HistoryItem.Resend = true;
					++SendingRepState->NumNaks;
				}
			}

			// Go over our Custom Delta Properties and update their retirements
			for (int32 i = SendingRepState->Retirement.Num() - 1; i >= 0; i--)
			{
				FPropertyRetirement& Retirement = SendingRepState->Retirement[i];
				ValidateRetirementHistory(Retirement, Object);

				// If this is a dynamic array property, we have to look through the list of retirement records to see if we need to reset the base state
				FPropertyRetirement* Rec = Retirement.Next; // Retirement[i] is head and not actually used in this case
				uint32 LastAcknowledged = 0;
				while (Rec != nullptr)
				{
					if (NakPacketId > Rec->OutPacketIdRange.Last)
					{
						// We can assume this means this record's packet was ack'd, so we can get rid of the old state
						check(Retirement.Next == Rec);
						Retirement.Next = Rec->Next;
						LastAcknowledged = Rec->FastArrayChangelistHistory;
						delete Rec;
						Rec = Retirement.Next;
						continue;
					}
					else if (NakPacketId >= Rec->OutPacketIdRange.First && NakPacketId <= Rec->OutPacketIdRange.Last)
					{
						UE_LOGF(LogNet, VeryVerbose, "Restoring Previous Base State of dynamic property. Channel: %ls, NakId: %d, First: %d, Last: %d, Address: %ls)", *OwningChannel->Describe(), NakPacketId, Rec->OutPacketIdRange.First, Rec->OutPacketIdRange.Last, *Connection->LowLevelGetRemoteAddress(true));

						// The Nack'd packet did update this property, so we need to replace the buffer in RecentDynamic
						// with the buffer we used to create this update (which was dropped), so that the update will be recreated on the next replicate actor
						
						if (LastAcknowledged != 0 && Rec->DynamicState.IsValid())
						{
							Rec->DynamicState->SetLastAckedHistory(LastAcknowledged);
						}

						SendingRepState->RecentCustomDeltaState[i] = Rec->DynamicState;

						// We can get rid of the rest of the saved off base states since we will be regenerating these updates on the next replicate actor
						while (Rec != nullptr)
						{
							FPropertyRetirement * DeleteNext = Rec->Next;
							delete Rec;
							Rec = DeleteNext;
						}

						// Finished
						Retirement.Next = nullptr;
						break;
					}
					Rec = Rec->Next;
				}

				ValidateRetirementHistory(Retirement, Object);
			}
		}
	}
}

bool FObjectReplicator::ReceivedBunch(FNetBitReader& Bunch, const FReplicationFlags& RepFlags, const bool bHasRepLayout, bool& bOutHasUnmapped)
{
	check(RepLayout);

	UObject* Object = GetObject();

	if (!Object)
	{
		UE_LOGF(LogNet, Verbose, "ReceivedBunch: Object == nullptr");
		return false;
	}

	UNetDriver* const ConnectionNetDriver = Connection->GetDriver();
	UPackageMap* const PackageMap = Connection->PackageMap;

	const bool bIsServer = ConnectionNetDriver->IsServer();
	const bool bCanDelayRPCs = (CVarDelayUnmappedRPCs.GetValueOnGameThread() > 0) && !bIsServer;
    const uint32 DriverReplicationFrame = ConnectionNetDriver->ReplicationFrame;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	const bool bQueueRpcForTransactions = ConnectionNetDriver->ShouldReplicateTransactionally() && (CVarQueueRpcsForTransactions.GetValueOnGameThread() > 0);
#else
	constexpr bool bQueueRpcForTransactions = false;
#endif

	const FClassNetCache* const ClassCache = ConnectionNetDriver->NetCache->GetClassNetCache(ObjectClass);

	if (!ClassCache)
	{
		UE_LOGF(LogNet, Error, "ReceivedBunch: ClassCache == nullptr: %ls", *Object->GetFullName());
		return false;
	}

	const FRepLayout& LocalRepLayout = *RepLayout;
	bool bGuidsChanged = false;
	FReceivingRepState* ReceivingRepState = RepState->GetReceivingRepState();

	// Handle replayout properties
	if (bHasRepLayout)
	{
		// Server shouldn't receive properties.
		if (bIsServer)
		{
			UE_LOGF(LogNet, Error, "Server received RepLayout properties: %ls", *Object->GetFullName());
			return false;
		}

		if (!bHasReplicatedProperties)
		{
			// Persistent, not reset until PostNetReceive is called
			bHasReplicatedProperties = true;
			PreNetReceive();
		}

		EReceivePropertiesFlags ReceivePropFlags = EReceivePropertiesFlags::None;

		if (ConnectionNetDriver->ShouldReceiveRepNotifiesForObject(Object))
		{
			ReceivePropFlags |= EReceivePropertiesFlags::RepNotifies;
		}

		if (RepFlags.bSkipRoleSwap)
		{
			ReceivePropFlags |= EReceivePropertiesFlags::SkipRoleSwap;
		}

		bool bLocalHasUnmapped = false;

		if (!LocalRepLayout.ReceiveProperties(OwningChannel, ObjectClass, RepState->GetReceivingRepState(), Object, Bunch, bLocalHasUnmapped, bGuidsChanged, ReceivePropFlags))
		{
			UE_LOGF(LogRep, Error, "RepLayout->ReceiveProperties FAILED: %ls", *Object->GetFullName());
			return false;
		}

		bOutHasUnmapped |= bLocalHasUnmapped;
	}

	FNetFieldExportGroup* NetFieldExportGroup = OwningChannel->GetNetFieldExportGroupForClassNetCache(ObjectClass);

	FNetBitReader Reader( Bunch.PackageMap );

	// Read fields from stream
	const FFieldNetCache * FieldCache = nullptr;
	
	// TODO:	As of now, we replicate all of our Custom Delta Properties immediately after our normal properties.
	//			An optimization could be made here in the future if we replicated / received Custom Delta Properties in RepLayout
	//			immediately with normal properties.
	//
	//			For the Standard case, we expect the RepLayout to be identical on Client and Server.
	//				If the RepLayout doesn't have any Custom Delta Properties, everything stays as it is now.
	//				If the RepLayout does have Custom Delta Properties, then:
	//					1. We replicate a single bit indicating whether or not any were actually sent.
	//					2. We replicate a packed int specifying the number of custom delta properties (if any were sent).
	//					3. We replicate the Header and Payloads as normal.
	//				This may increase bandwidth slightly, but it's likely negligible.
	//
	//			For the Backwards Compatible path, we do the above, except we always send the bit flag, and the count when set.
	//				In that way, if Custom Delta Properties are added or removed, we can always rely on the bit field to try and
	//				read them, and then throw them away if they are incompatible.
	//
	//			In both described cases, we could remove the first cast to a struct property below, and flags checks on the properties
	//			as we could instead use the RepLayout cached command flags which would hopefully reduce cache misses.
	//			This also means that we could leverage the bIsServer and bHasReplicatedProperties that have already taken place.
	//
	//			If we want maintain compatibility with older builds (mostly for replays), we could leave the branch in here for now
	//			but short circuit it with a net version check, still allowing us to skip the cast in new versions.
	//
	//			This also becomes more convenient when we merge RepNotify handling.

	FNetSerializeCB NetSerializeCB(ConnectionNetDriver);

	// Read each property/function blob into Reader (so we've safely jumped over this data in the Bunch/stream at this point)
	while (true)
	{
		UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(FieldHeaderAndPayloadScope, FName(), Bunch, OwningChannel->Connection->GetInTraceCollector(), ENetTraceVerbosity::Trace);

		if (!OwningChannel->ReadFieldHeaderAndPayload(Object, ClassCache, NetFieldExportGroup, Bunch, &FieldCache, Reader))
		{
			break;
		}

		// Instead of using a "sub-collector" we inject an offset and populates packet content events by using the incoming collector assuming that we read data from packet sequentially
		UE_NET_TRACE_OFFSET_SCOPE(Bunch.GetPosBits() - Reader.GetNumBits(), OwningChannel->Connection->GetInTraceCollector());

		if (UNLIKELY(Bunch.IsError()))
		{
			UE_LOGF(LogNet, Error, "ReceivedBunch: Error reading field: %ls", *Object->GetFullName());
			return false;
		}

		else if (UNLIKELY(FieldCache == nullptr))
		{
			UE_LOGF(LogNet, Warning, "ReceivedBunch: FieldCache == nullptr: %ls", *Object->GetFullName());
			continue;
		}

		else if (UNLIKELY(FieldCache->bIncompatible))
		{
			// We've already warned about this property once, so no need to continue to do so
			UE_LOGF(LogNet, Verbose, "ReceivedBunch: FieldCache->bIncompatible == true. Object: %ls, Field: %ls", *Object->GetFullName(), *FieldCache->Field.GetFName().ToString());
			continue;
		}

		UE_NET_TRACE_SET_SCOPE_NAME(FieldHeaderAndPayloadScope, FieldCache->Field.GetFName());

		// Handle property
		if (FStructProperty* ReplicatedProp = CastField<FStructProperty>(FieldCache->Field.ToField()))
		{
			// Server shouldn't receive properties.
			if (bIsServer)
			{
				UE_LOGF(LogNet, Error, "Server received unwanted property value %ls in %ls", *ReplicatedProp->GetName(), *Object->GetFullName());
				return false;
			}

			// Call PreNetReceive if we haven't yet
			if (!bHasReplicatedProperties)
			{
				// Persistent, not reset until PostNetReceive is called
				bHasReplicatedProperties = true;
				PreNetReceive();
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			{
				FString DebugPropertyStr = CVarNetReplicationDebugProperty.GetValueOnAnyThread();
				if (!DebugPropertyStr.IsEmpty() && ReplicatedProp->GetName().Contains(DebugPropertyStr))
				{
					UE_LOGF(LogRep, Log, "Replicating Property[%d] %ls on %ls", ReplicatedProp->RepIndex, *ReplicatedProp->GetName(), *Object->GetName());
				}
			}
#endif

			FNetDeltaSerializeInfo Parms;
			Parms.Map = PackageMap;
			Parms.Reader = &Reader;
			Parms.NetSerializeCB = &NetSerializeCB;
			Parms.Connection = Connection;
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
			if (ConnectionNetDriver && ConnectionNetDriver->GetWorld())
			{
				Parms.bCallFastArrayClientCallbacks = ConnectionNetDriver->GetWorld()->IsActorTickAndUserCallbacksEnabled();
			}
#endif
			Parms.bInternalAck = Connection->IsInternalAck();
			Parms.Object = Object;

			if (!FNetSerializeCB::ReceiveCustomDeltaProperty(LocalRepLayout, ReceivingRepState, Parms, ReplicatedProp))
			{
				// Should have already logged the error.
				if (bIsServer)
				{
					return false;
				}

				FieldCache->bIncompatible = true;
				continue;
			}

			if (Parms.bOutHasMoreUnmapped)
			{
				bOutHasUnmapped = true;
			}

			if (Parms.bGuidListsChanged)
			{
				bGuidsChanged = true;
			}

			// Successfully received it.
			UE_LOGF(LogRepTraffic, Log, " %ls - %ls", *Object->GetName(), *Parms.DebugName);
		}
		// Handle function call
		else if ( Cast< UFunction >( FieldCache->Field.ToUObject() ) )
		{
			ESkipRpcBehavior SkipRpcBehavior = bCanDelayRPCs ? ESkipRpcBehavior::SkipIfNotReady : ESkipRpcBehavior::NeverSkip;
			if (bQueueRpcForTransactions)
			{
				SkipRpcBehavior = ESkipRpcBehavior::AlwaysSkip;

				// To debug why RPC property deserialization is aborting:
				//UE_AUTORTFM_ONABORT()
				//{
				//	ensureMsgf(false, TEXT("Expecting that we never abort when we are using queued transactional RPCs"));
				//};
			}

			bool bSkippedRpcExec = false;
			TSet<FNetworkGUID> UnmappedGuids;
			bool bSuccess = ReceivedRPC(Reader, UnmappedGuids, bSkippedRpcExec, RepFlags, FieldCache, SkipRpcBehavior);

			if (!bSuccess)
			{
				return false;
			}
			else if (bSkippedRpcExec)
			{
				if (bQueueRpcForTransactions && UnmappedGuids.IsEmpty())
				{
					// We are skipping now and executing later (PostTickDispatch)
					PendingTransactionalRPCs.Emplace(FieldCache, RepFlags, Reader, DriverReplicationFrame, UnmappedGuids);
					ConnectionNetDriver->ReplicatorsWithPendingTransactions.Add(this);
				}
				else
				{
					// Unmapped GUIDs: Either due to not receiving a GUID yet from replication (client only),
					// or this can also happen on the client & server due to async loading.
					PendingLocalRPCs.Emplace(FieldCache, RepFlags, Reader, DriverReplicationFrame, UnmappedGuids);
					bOutHasUnmapped = true;
					bGuidsChanged = true;
					bForceUpdateUnmapped = true;
				}
			}
			else if (!IsValid(Object))
			{
				// replicated function destroyed Object
				return true;
			}
		}
		else
		{
			UE_LOGF( LogRep, Error, "ReceivedBunch: Invalid replicated field %i in %ls", FieldCache->FieldNetIndex, *Object->GetFullName() );
			return false;
		}
	}
	
	// If guids changed, then rebuild acceleration tables
	if (bGuidsChanged)
	{
		UpdateGuidToReplicatorMap();
	}

	return true;
}

#define HANDLE_INCOMPATIBLE_RPC			\
	if ( bIsServer )					\
	{									\
		return false;					\
	}									\
	FieldCache->bIncompatible = true;	\
	return true;						\


bool GReceiveRPCTimingEnabled = false;
struct FScopedRPCTimingTracker
{
	FScopedRPCTimingTracker(UFunction* InFunction, UNetConnection* InConnection)
		: Connection(InConnection)
		, Function(InFunction)
	{
		if (GReceiveRPCTimingEnabled)
		{
			StartTime = FPlatformTime::Seconds();
		}

		ActiveTrackers.Add(this);
	};

	~FScopedRPCTimingTracker()
	{
		if (RPCDoS != nullptr && RPCDoS->IsRPCDoSDetectionEnabled())
		{
			RPCDoS->PostReceivedRPC();
		}

		ActiveTrackers.RemoveSingleSwap(this);
		if (GReceiveRPCTimingEnabled)
		{
			const double Elapsed = FPlatformTime::Seconds() - StartTime;
			Connection->Driver->NotifyRPCProcessed(Function, Connection, Elapsed);
		}
	}

	UNetConnection* Connection = nullptr;
	UFunction* Function = nullptr;
	FRPCDoSDetection* RPCDoS = nullptr;
	double StartTime = 0.0;

	static TArray<FScopedRPCTimingTracker*> ActiveTrackers;
};

TArray<FScopedRPCTimingTracker*> FScopedRPCTimingTracker::ActiveTrackers;

/** Return the list of UFunctions currently tracked or  */
ENGINE_API TArray<UFunction*> FindScopedRPCTrackers(UNetConnection* Connection = nullptr)
{
	TArray<UFunction*> FuncList;
	for (const FScopedRPCTimingTracker* TestTracker : FScopedRPCTimingTracker::ActiveTrackers)
	{
		if (TestTracker && (Connection == nullptr || TestTracker->Connection == Connection))
		{
			FuncList.Add(TestTracker->Function);
		}
	}

	return FuncList;
}

bool FObjectReplicator::ReceivedRPC(FNetBitReader& Reader, TSet<FNetworkGUID>& UnmappedGuids, bool& bOutSkippedRpcExec, const FReplicationFlags& RepFlags, const FFieldNetCache* FieldCache, ESkipRpcBehavior SkipRpcBehavior)
{
	check(FieldCache);
	bOutSkippedRpcExec = false;

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(HandleRPC);
	const bool bIsServer = Connection->Driver->IsServer();
	UObject* Object = GetObject();
	FName FunctionName = FieldCache->Field.GetFName();
	UFunction* Function = Object->FindFunction(FunctionName);
	FRPCDoSDetection* RPCDoS = Connection->GetRPCDoS();
	FScopedRPCTimingTracker ScopedTracker(Function, Connection);
	SCOPE_CYCLE_COUNTER(STAT_NetReceiveRPC);
	SCOPE_CYCLE_UOBJECT(Function, Function);

	if (Function == nullptr)
	{
		UE_LOGF(LogNet, Error, "ReceivedRPC: Function not found. Object: %ls, Function: %ls", *Object->GetFullName(), *FunctionName.ToString());
		HANDLE_INCOMPATIBLE_RPC
	}

	if ((Function->FunctionFlags & FUNC_Net) == 0)
	{
		UE_LOGF(LogRep, Error, "Rejected non RPC function. Object: %ls, Function: %ls", *Object->GetFullName(), *FunctionName.ToString());
		HANDLE_INCOMPATIBLE_RPC
	}

	// If NetServer, NetClient, or NetMulticast flags are present, filter the RPC based on them. If not, accept the RPC for servers and clients.
	if ((Function->FunctionFlags & FUNC_NetServer) && !bIsServer)
	{
		UE_LOGF(LogRep, Error, "Rejected server RPC function due to access rights. Object: %ls, Function: %ls", *Object->GetFullName(), *FunctionName.ToString());
		HANDLE_INCOMPATIBLE_RPC
	}

	if ((Function->FunctionFlags & (FUNC_NetClient | FUNC_NetMulticast)) && bIsServer)
	{
		UE_LOGF(LogRep, Error, "Rejected client RPC function due to access rights. Object: %ls, Function: %ls", *Object->GetFullName(), *FunctionName.ToString());
		HANDLE_INCOMPATIBLE_RPC
	}


	if (RPCDoS != nullptr && bIsServer && RPCDoS->IsRPCDoSDetectionEnabled() && !Connection->IsReplay())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RPCDosDetection);

		ScopedTracker.RPCDoS = RPCDoS;

		if (UNLIKELY(RPCDoS->ShouldMonitorReceivedRPC()))
		{
			ERPCNotifyResult Result = RPCDoS->NotifyReceivedRPC(Object, Function, FunctionName);

			if (UNLIKELY(Result == ERPCNotifyResult::BlockRPC))
			{
				RPCDoS->TrackBlockedRPC(FunctionName);

				return true;
			}
		}
		else
		{
			RPCDoS->LightweightReceivedRPC(Function, FunctionName);
		}
	}

	UE_LOGF(LogRepTraffic, Log, "      Received RPC: %ls", *FunctionName.ToString());

	// validate that the function is callable here
	// we are client or net owner and shouldn't be ignoring rpcs
	const bool bCanExecute = Connection->Driver->ShouldCallRemoteFunction(Object, Function, RepFlags);

	if (bCanExecute)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EvaluateCanExecute);

		// Get the parameters.
		FMemMark Mark(FMemStack::Get());
		uint8* Parms = new(FMemStack::Get(), MEM_Zeroed, Function->ParmsSize)uint8;

		// Use the replication layout to receive the rpc parameter values
		UFunction* LayoutFunction = Function;
		while (LayoutFunction->GetSuperFunction())
		{
			LayoutFunction = LayoutFunction->GetSuperFunction();
		}

		TSharedPtr<FRepLayout> FuncRepLayout = Connection->Driver->GetFunctionRepLayout(LayoutFunction);
		if (!FuncRepLayout.IsValid())
		{
			UE_LOGF(LogRep, Error, "ReceivedRPC: GetFunctionRepLayout returned an invalid layout.");
			return false;
		}

		FuncRepLayout->ReceivePropertiesForRPC(Object, LayoutFunction, OwningChannel, Reader, Parms, UnmappedGuids);

		if (Reader.IsError())
		{
			UE_LOGF(LogRep, Error, "ReceivedRPC: ReceivePropertiesForRPC - Reader.IsError() == true: Function: %ls, Object: %ls", *FunctionName.ToString(), *Object->GetFullName());
			HANDLE_INCOMPATIBLE_RPC
		}

		if (Reader.GetBitsLeft() != 0)
		{
			UE_LOGF(LogNet, Error, "ReceivedRPC: ReceivePropertiesForRPC - Mismatch read. Function: %ls, Object: %ls", *FunctionName.ToString(), *Object->GetFullName());
			HANDLE_INCOMPATIBLE_RPC
		}

		RPC_ResetLastFailedReason();

		// See if the caller wants us to potentially skip the execution of this RPC
		const bool bCanDelayUnmapped = (SkipRpcBehavior == ESkipRpcBehavior::SkipIfNotReady) && (Function->FunctionFlags & FUNC_NetReliable);
		const bool bDelayUnmappedRPCs = bCanDelayUnmapped && (UnmappedGuids.Num() > 0 || PendingLocalRPCs.Num() > 0);
		bOutSkippedRpcExec = (bDelayUnmappedRPCs || SkipRpcBehavior == ESkipRpcBehavior::AlwaysSkip);

		if (!bOutSkippedRpcExec)
		{
			AActor* OwningActor = OwningChannel->Actor;
			UObject* const SubObject = Object != OwningChannel->Actor ? Object : nullptr;

			// Forward the function call.
			Connection->Driver->ForwardRemoteFunction(OwningActor, SubObject, Function, Parms);

			// Reset errors from replay driver
			RPC_ResetLastFailedReason();

			{
				// Call the function.
				if (Connection->Driver->IsExecuteRPCFunctionsEnabled())
				{
					CallProcessEventForReceivedRPC(Object, Function, Parms);
				}
			}
		}

		// Destroy the parameters.
		// warning: highly dependent on UObject::ProcessEvent freeing of parms!
		for (TFieldIterator<FProperty> It(Function); It && (It->PropertyFlags & (CPF_Parm | CPF_ReturnParm)) == CPF_Parm; ++It)
		{
			It->DestroyValue_InContainer(Parms);
		}

		Mark.Pop();

		if (RPC_GetLastFailedReason() != nullptr)
		{
			UE_LOGF(LogRep, Error, "ReceivedRPC: RPC_GetLastFailedReason: %ls", RPC_GetLastFailedReason());
			return false;
		}
	}
	else
	{
		UE_LOGF(LogRep, Verbose, "Rejected unwanted function %ls in %ls", *FunctionName.ToString(), *Object->GetFullName());
	}

	return true;
}

void FObjectReplicator::CallProcessEventForReceivedRPC(UObject* Object, UFunction* Function, uint8* Params)
{
	UE::Net::FScopedNetContextRPC CallingRPC;
	UE::Net::Private::FScopedRemoteRPCMode ReceivingRemoteRPC(Function, UE::Net::Private::ERemoteFunctionMode::Receiving);

	Object->ProcessEvent(Function, Params);
}

void FObjectReplicator::UpdateGuidToReplicatorMap()
{
	LLM_SCOPE_BYTAG(NetObjReplicator);
	SCOPE_CYCLE_COUNTER(STAT_NetUpdateGuidToReplicatorMap);

	if (Connection->Driver->IsServer())
	{
		return;
	}

	TSet<FNetworkGUID> LocalReferencedGuids;
	int32 LocalTrackedGuidMemoryBytes = 0;

	check(RepLayout);

	const FRepLayout& LocalRepLayout = *RepLayout;

	// Gather guids on rep layout
	if (RepState.IsValid())
	{
		FNetSerializeCB NetSerializeCB(Connection->Driver);

		FNetDeltaSerializeInfo Parms;
		Parms.Connection = Connection;
		Parms.NetSerializeCB = &NetSerializeCB;
		Parms.GatherGuidReferences = &LocalReferencedGuids;
		Parms.TrackedGuidMemoryBytes = &LocalTrackedGuidMemoryBytes;
		Parms.Object = GetObject();
		Parms.bInternalAck = Connection->IsInternalAck();

		LocalRepLayout.GatherGuidReferences(RepState->GetReceivingRepState(), Parms, LocalReferencedGuids, LocalTrackedGuidMemoryBytes);
	}

	// Gather RPC guids
	for (const FRPCPendingLocalCall& PendingRPC : PendingLocalRPCs)
	{
		for (const FNetworkGUID& NetGuid : PendingRPC.UnmappedGuids)
		{
			LocalReferencedGuids.Add(NetGuid);

			LocalTrackedGuidMemoryBytes += PendingRPC.UnmappedGuids.GetAllocatedSize();
			LocalTrackedGuidMemoryBytes += PendingRPC.Buffer.Num();
		}
	}

	// Go over all referenced guids, and make sure we're tracking them in the GuidToReplicatorMap
	for (const FNetworkGUID& GUID : LocalReferencedGuids)
	{
		if (!ReferencedGuids.Contains(GUID))
		{
			LLM_SCOPE_BYTAG(NetDriver);
			Connection->Driver->GuidToReplicatorMap.FindOrAdd(GUID).Add(this);
		}
	}

	// Remove any guids that we were previously tracking but no longer should
	for (const FNetworkGUID& GUID : ReferencedGuids)
	{
		if (!LocalReferencedGuids.Contains(GUID))
		{
			TSet<FObjectReplicator*>& Replicators = Connection->Driver->GuidToReplicatorMap.FindChecked(GUID);

			Replicators.Remove(this);

			if (Replicators.Num() == 0)
			{
				Connection->Driver->GuidToReplicatorMap.Remove(GUID);
			}
		}
	}

	Connection->Driver->TotalTrackedGuidMemoryBytes -= TrackedGuidMemoryBytes;
	TrackedGuidMemoryBytes = LocalTrackedGuidMemoryBytes;
	Connection->Driver->TotalTrackedGuidMemoryBytes += TrackedGuidMemoryBytes;

	ReferencedGuids = MoveTemp(LocalReferencedGuids);
}

bool FObjectReplicator::MoveMappedObjectToUnmapped(const FNetworkGUID& GUID)
{
	check(RepLayout);
	const FRepLayout& LocalRepLayout = *RepLayout;

	FNetSerializeCB NetSerializeCB(Connection->Driver);
	FNetDeltaSerializeInfo Parms;
	Parms.Connection = Connection;
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	if (Connection)
	{
		UNetDriver* ConnectionDriver = Connection->GetDriver();
		if (ConnectionDriver && ConnectionDriver->GetWorld())
		{
			Parms.bCallFastArrayClientCallbacks = ConnectionDriver->GetWorld()->IsActorTickAndUserCallbacksEnabled();
		}
	}
#endif
	Parms.bInternalAck = Connection->IsInternalAck();
	Parms.Map = Connection->PackageMap;
	Parms.Object = GetObject();
	Parms.NetSerializeCB = &NetSerializeCB;
	Parms.MoveGuidToUnmapped = &GUID;

	return LocalRepLayout.MoveMappedObjectToUnmapped(RepState->GetReceivingRepState(), Parms, GUID);
}

void FObjectReplicator::PostReceivedBunch()
{
	if ( GetObject() == nullptr )
	{
		UE_LOGF(LogNet, Verbose, "PostReceivedBunch: Object == nullptr");
		return;
	}

	// Call PostNetReceive
	const bool bIsServer = (OwningChannel->Connection->Driver->ServerConnection == nullptr);
	if (!bIsServer && bHasReplicatedProperties)
	{
		PostNetReceive();
		bHasReplicatedProperties = false;
	}

	// Call RepNotifies
	CallRepNotifies(true);
}

static FORCEINLINE FPropertyRetirement** UpdateAckedRetirements(
	FPropertyRetirement& Retire,
	uint32& LastAcknowledged,
	const int32 OutAckPacketId,
	const UObject* Object)
{
	ValidateRetirementHistory(Retire, Object);

	FPropertyRetirement** Rec = &Retire.Next; // Note the first element is 'head' that we don't actually use

	LastAcknowledged = 0;
	while (*Rec != nullptr)
	{
		if (OutAckPacketId >= (*Rec)->OutPacketIdRange.Last)
		{
			UE_LOGF(LogRepTraffic, Verbose, "Deleting Property Record (%d >= %d)", OutAckPacketId, (*Rec)->OutPacketIdRange.Last);

			// They've ack'd this packet so we can ditch this record (easier to do it here than look for these every Ack)
			FPropertyRetirement* ToDelete = *Rec;
			check(Retire.Next == ToDelete); // This should only be able to happen to the first record in the list
			Retire.Next = ToDelete->Next;
			Rec = &Retire.Next;

			LastAcknowledged = ToDelete->FastArrayChangelistHistory;
			delete ToDelete;
			continue;
		}

		Rec = &(*Rec)->Next;
	}

	return Rec;
}

void FObjectReplicator::ReplicateCustomDeltaProperties( FNetBitWriter & Bunch, FReplicationFlags RepFlags, bool& bSkippedPropertyCondition)
{
	check(RepLayout);
	const FRepLayout& LocalRepLayout = *RepLayout;

	bSkippedPropertyCondition = false;

	const int32 NumLifetimeCustomDeltaProperties = FNetSerializeCB::GetNumLifetimeCustomDeltaProperties(LocalRepLayout);

	if (NumLifetimeCustomDeltaProperties <= 0)
	{
		// No custom properties
		return;
	}

	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_NetReplicateCustomDeltaPropTime, GUseDetailedScopeCounters);

	// TODO: See comments in ReceivedBunch. This code should get merged into RepLayout, to help optimize
	//			the receiving end, and make things more consistent.

	UObject* Object = GetObject();
	FSendingRepState* SendingRepState = RepState->GetSendingRepState();

	check(Object);
	check(OwningChannel);
	check(Connection == OwningChannel->Connection);

	TArray<TSharedPtr<INetDeltaBaseState>>& UsingCustomDeltaStates =
		EResendAllDataState::None == Connection->ResendAllDataState ? SendingRepState->RecentCustomDeltaState :
		EResendAllDataState::SinceOpen == Connection->ResendAllDataState ? SendingRepState->CDOCustomDeltaState :
		SendingRepState->CheckpointCustomDeltaState;

	FNetSerializeCB::PreSendCustomDeltaProperties(LocalRepLayout, Object, Connection, *ChangelistMgr, UsingCustomDeltaStates);

	ON_SCOPE_EXIT
	{
		FNetSerializeCB::PostSendCustomDeltaProperties(LocalRepLayout, Object, Connection, *ChangelistMgr, UsingCustomDeltaStates);
	};

	// Initialize a map of which conditions are valid
	const TStaticBitArray<COND_Max> ConditionMap = UE::Net::BuildConditionMapFromRepFlags(RepFlags);

	// Make sure net field export group is registered
	FNetFieldExportGroup* NetFieldExportGroup = OwningChannel->GetOrCreateNetFieldExportGroupForClassNetCache( Object );

	FNetBitWriter TempBitWriter( Connection->PackageMap, 1024 );

#if UE_NET_TRACE_ENABLED
	SetTraceCollector(TempBitWriter, GetTraceCollector(Bunch) ? UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace) : nullptr);
    ON_SCOPE_EXIT { UE_NET_TRACE_DESTROY_COLLECTOR(GetTraceCollector(TempBitWriter)); };
#endif
	
	const bool bIsConnectionInternalAck = Connection->IsInternalAck();

	// We must forward the current NetToken export context from the bunch to the temporary NetBitWriter into which this data will be written
	TempBitWriter.NetTokenExportContext.Set(Bunch.NetTokenExportContext.Get());
	
	// Replicate those properties.
	for (uint16 CustomDeltaProperty = 0; CustomDeltaProperty < NumLifetimeCustomDeltaProperties; ++CustomDeltaProperty)
	{
		ELifetimeCondition RepCondition = FNetSerializeCB::GetLifetimeCustomDeltaPropertyCondition(LocalRepLayout, CustomDeltaProperty);
		FProperty* Property = FNetSerializeCB::GetLifetimeCustomDeltaProperty(LocalRepLayout, CustomDeltaProperty);

		if (RepCondition == COND_Dynamic)
		{
			if (const FRepChangedPropertyTracker* RepChangedPropertyTracker = SendingRepState->RepChangedPropertyTracker.Get())
			{
				RepCondition = RepChangedPropertyTracker->GetDynamicCondition(Property->RepIndex);
			}
		}

		check(RepCondition >= 0 && RepCondition < COND_Max);

		if (!ConditionMap[RepCondition])
		{
			// We didn't pass the condition so don't replicate us
			bSkippedPropertyCondition = true;
			continue;
		}

		// If this is a dynamic array, we do the delta here
		TSharedPtr<INetDeltaBaseState> NewState;

		TempBitWriter.Reset();
#if UE_NET_TRACE_ENABLED
		if (FNetTraceCollector* TraceCollection = GetTraceCollector(TempBitWriter))
		{
			TraceCollection->Reset();
		}
#endif

		TSharedPtr<INetDeltaBaseState>& OldState = UsingCustomDeltaStates[CustomDeltaProperty];

		if (Connection->ResendAllDataState != EResendAllDataState::None)
		{
			if (Connection->ResendAllDataState == EResendAllDataState::SinceCheckpoint)
			{
				if (!SendCustomDeltaProperty(Object, CustomDeltaProperty, TempBitWriter, NewState, OldState))
				{
					continue;
				}

				// update checkpoint with new state
				OldState = NewState;
			}
			else
			{
				// If we are resending data since open, we don't want to affect the current state of channel/replication, so just do the minimum and send the data, and return
				// In this case, we'll send all of the properties since the CDO, so use the initial CDO delta state
				if (!SendCustomDeltaProperty(Object, CustomDeltaProperty, TempBitWriter, NewState, OldState))
				{
					continue;
				}
			}

			// Write property header and payload to the bunch
			WritePropertyHeaderAndPayload(Object, Property, NetFieldExportGroup, Bunch, TempBitWriter);

			continue;
		}

		FPropertyRetirement** LastNext = nullptr;

		if (!bIsConnectionInternalAck)
		{
			// Get info.
			FPropertyRetirement& Retire = SendingRepState->Retirement[CustomDeltaProperty];

			// Update Retirement records with this new state so we can handle packet drops.
			// LastNext will be pointer to the last "Next" pointer in the list (so pointer to a pointer)
			uint32 LastAcknowledged = 0;
			LastNext = UpdateAckedRetirements(Retire, LastAcknowledged, Connection->OutAckPacketId, Object);

			if (LastAcknowledged != 0 && OldState.IsValid())
			{
				OldState->SetLastAckedHistory(LastAcknowledged);
			}

			check(LastNext != nullptr);
			check(*LastNext == nullptr);

			ValidateRetirementHistory(Retire, Object);
		}

		//-----------------------------------------
		//	Do delta serialization on dynamic properties
		//-----------------------------------------
		const bool WroteSomething = SendCustomDeltaProperty(Object, CustomDeltaProperty, TempBitWriter, NewState, OldState);

		if ( !WroteSomething )
		{
			continue;
		}

		if (!bIsConnectionInternalAck)
		{
			*LastNext = new FPropertyRetirement();

			// Remember what the old state was at this point in time.  If we get a nak, we will need to revert back to this.
			(*LastNext)->DynamicState = OldState;

			// Using NewState's ChangelistHistory seems counter intuitive at first, because it *seems* like we should be using the history from the OldState.
			// However, PropertyRetirements associate the OldState with the state of replication **in case of a NAK**.
			// So, in the case of an ACK, we know that we no longer need to hold onto that OldState because the client has received
			// the NewState (which will become our OldState the next time we send something).
			if (NewState.IsValid())
			{
				(*LastNext)->FastArrayChangelistHistory = NewState->GetChangelistHistory();
			}
		}

		// Save NewState into the RecentCustomDeltaState array (old state is a reference into our RecentCustomDeltaState map)
		OldState = NewState; 

		// Write property header and payload to the bunch
		WritePropertyHeaderAndPayload(Object, Property, NetFieldExportGroup, Bunch, TempBitWriter);

		NETWORK_PROFILER(GNetworkProfiler.TrackReplicateProperty(Property, TempBitWriter.GetNumBits(), Connection));
	}
}

bool FObjectReplicator::CanSkipUpdate(FReplicationFlags RepFlags)
{
#if WITH_PUSH_MODEL
	if (!bCanUseNonDirtyOptimization)
	{
		// This class must always replicate
		return false;
	}

	const bool bHasRPCQueued = RemoteFunctions && RemoteFunctions->GetNumBits() >= 0;
	if (bHasRPCQueued)
	{
		return false;
	}

	const bool bHasNoRepLayout = RepLayout->IsEmpty();
	if (bHasNoRepLayout)
	{
		// No properties to replicate and no RPCs queued, let's skip!
#if CSV_PROFILER_STATS
		++GNumSkippedObjectEmptyUpdates;
#endif

		bLastUpdateEmpty = true;
		return true;
	}

	const bool bIsNetInitial = RepFlags.bNetInitial;
	if (bIsNetInitial)
	{
		return false;
	}

	const FSendingRepState& SendingRepState = *RepState->GetSendingRepState();
	const FRepChangelistState& RepChangelistState = *ChangelistMgr->GetRepChangelistState();

	bool bCanSkip = true;

	// Is the pushmodel handle properly assigned.
	bCanSkip = bCanSkip && RepChangelistState.HasValidPushModelHandle();

	// Have the RepFlags changed ?
	bCanSkip = bCanSkip && SendingRepState.RepFlags.Value == RepFlags.Value; 

	// Any Naks to handle ?
	bCanSkip = bCanSkip && SendingRepState.NumNaks == 0;

	// Have we compared the properties twice ?
	bCanSkip = bCanSkip && SendingRepState.LastCompareIndex > 1;

	// Do we have open Ack's ?
	bCanSkip = bCanSkip && !(SendingRepState.bOpenAckedCalled && SendingRepState.PreOpenAckHistory.Num() > 0);

	// Any changelists to send ?
	bCanSkip = bCanSkip && SendingRepState.LastChangelistIndex == RepChangelistState.HistoryEnd;

	// Is the changelist history fully acknowledged ?
	bCanSkip = bCanSkip && SendingRepState.HistoryStart == SendingRepState.HistoryEnd;

	// Are we resending data for replay ?
	bCanSkip = bCanSkip && Connection->ResendAllDataState == EResendAllDataState::None;

	// Are we forcing a compare property ?
	bCanSkip = bCanSkip && !OwningChannel->bForceCompareProperties;

	// Are any CustomDelta properties dirty ?
	bCanSkip = bCanSkip && (RepChangelistState.CustomDeltaChangeIndex == SendingRepState.CustomDeltaChangeIndex && !SendingRepState.HasAnyPendingRetirements());

	// Are any replicated properties dirty ?
	bCanSkip = bCanSkip && RepChangelistState.HasAnyDirtyProperties() == false;

	if (bCanSkip)
	{
#if CSV_PROFILER_STATS
		++GNumSkippedObjectEmptyUpdates;
#endif

		bLastUpdateEmpty = true;
	}

	return bCanSkip;
#else // WITH_PUSH_MODEL
	return false;
#endif
}

bool FObjectReplicator::ReplicateProperties(FOutBunch& Bunch, FReplicationFlags RepFlags, FNetBitWriter& Writer)
{
	LLM_SCOPE_BYTAG(NetObjReplicator);
	return ReplicateProperties_r(Bunch, RepFlags, Writer);
}

bool FObjectReplicator::ReplicateProperties(FOutBunch& Bunch, FReplicationFlags RepFlags)
{
	LLM_SCOPE_BYTAG(NetObjReplicator);
	FNetBitWriter Writer(Bunch.PackageMap, 8192);
	return ReplicateProperties_r(Bunch, RepFlags, Writer);
}

/** Replicates properties to the Bunch. Returns true if it wrote anything */
bool FObjectReplicator::ReplicateProperties_r(FOutBunch& Bunch, FReplicationFlags RepFlags, FNetBitWriter& Writer)
{
	UObject* Object = GetObject();

	if (Object == nullptr)
	{
		UE_LOGF(LogRep, Verbose, "ReplicateProperties: Object == nullptr");
		return false;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// some games ship checks() in Shipping so we cannot rely on DO_CHECK here, and these checks are in an extremely hot path
	check(OwningChannel);
	check(RepLayout.IsValid());
	check(RepState.IsValid());
	check(RepState->GetSendingRepState());
	check(ChangelistMgr.IsValid());
	check(ChangelistMgr->GetRepChangelistState() != nullptr);
	check((ChangelistMgr->GetRepChangelistState()->StaticBuffer.Num() == 0) == RepLayout->IsEmpty());
#endif

	UNetConnection* OwningChannelConnection = OwningChannel->Connection;

	// Pass on NetTokenExportContext
	Writer.NetTokenExportContext.Set(Bunch.NetTokenExportContext.Get());

#if UE_NET_TRACE_ENABLED
	// Create trace collector if tracing is enabled for the target bunch
	SetTraceCollector(Writer, GetTraceCollector(Bunch) ? UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace) : nullptr);
#endif

    ON_SCOPE_EXIT 
	{
#if UE_NET_TRACE_ENABLED		
		UE_NET_TRACE_DESTROY_COLLECTOR(GetTraceCollector(Writer)); 
#endif
		// Just to be safe, restore context on exit as we technically do not own the Writer.
		Writer.NetTokenExportContext.Set(nullptr);
	};
 
	// TODO: Maybe ReplicateProperties could just take the RepState, Changelist Manger, Writer, and OwningChannel
	//		and all the work could just be done in a single place.

	// Update change list (this will re-use work done by previous connections)

	const bool bUseCheckpointRepState = (Connection->ResendAllDataState == EResendAllDataState::SinceCheckpoint);

	if (bUseCheckpointRepState && !CheckpointRepState.IsValid())
	{
		TSharedPtr<FRepChangedPropertyTracker> RepChangedPropertyTracker = Connection->Driver->FindOrCreateRepChangedPropertyTracker(GetObject());

		CheckpointRepState = RepLayout->CreateRepState((const uint8*)Object, RepChangedPropertyTracker, ECreateRepStateFlags::SkipCreateReceivingState);
	}

	FSendingRepState* SendingRepState = (bUseCheckpointRepState && CheckpointRepState.IsValid()) ? CheckpointRepState->GetSendingRepState() : RepState->GetSendingRepState();

#if WITH_PUSH_MODEL
	FRepChangelistState* ChangelistState = ChangelistMgr->GetRepChangelistState();

	bool bHasDirtyCustomDeltaProperties = false;

	const UEPushModelPrivate::FPushModelPerNetDriverHandle Handle = ChangelistState->GetPushModelObjectHandle();
	if (Handle.IsValid())
	{
		if (UEPushModelPrivate::FPushModelPerNetDriverState* PushModelState = UEPushModelPrivate::GetPerNetDriverState(Handle))
		{
			const uint16 NumLifetimeCustomDeltaProperties = FNetSerializeCB::GetNumLifetimeCustomDeltaProperties(*RepLayout);

			for (uint16 CustomDeltaProperty = 0; CustomDeltaProperty < NumLifetimeCustomDeltaProperties; ++CustomDeltaProperty)
			{
				if (PushModelState->IsPropertyDirty(FNetSerializeCB::GetLifetimeCustomDeltaPropertyRepIndex(*RepLayout, CustomDeltaProperty)))
				{
					bHasDirtyCustomDeltaProperties = true;
					break;
				}
			}
		}
	}
#endif

	const ERepLayoutResult UpdateResult = FNetSerializeCB::UpdateChangelistMgr(*RepLayout, SendingRepState, *ChangelistMgr, Object, Connection->Driver->ReplicationFrame, RepFlags, OwningChannel->bForceCompareProperties || bUseCheckpointRepState);

	if (UNLIKELY(ERepLayoutResult::FatalError == UpdateResult))
	{
		Connection->SetPendingCloseDueToReplicationFailure();
		return false;
	}

	// Replicate properties in the layout
	const bool bHasRepLayout = RepLayout->ReplicateProperties(SendingRepState, ChangelistMgr->GetRepChangelistState(), (uint8*)Object, ObjectClass, OwningChannel, Writer, RepFlags);

	NETWORK_PROFILER(
		if (GNetworkProfiler.IsComparisonTrackingEnabled() && bHasRepLayout)
		{
			GNetworkProfiler.TrackReplicatePropertiesMetadata(RepLayout->GetOwner(), SendingRepState->InactiveParents, (Connection->ResendAllDataState != EResendAllDataState::None), Connection);
		}
	);

	// Replicate all the custom delta properties (fast arrays, etc)

	{
#if WITH_PUSH_MODEL
		const int32 WriterBits = Writer.GetNumBits();
#endif // WITH_PUSH_MODEL

		bool bSkippedPropertyCondition = false;
		ReplicateCustomDeltaProperties(Writer, RepFlags, bSkippedPropertyCondition);

#if WITH_PUSH_MODEL
		if (WriterBits != Writer.GetNumBits())
		{
			if (bHasDirtyCustomDeltaProperties)
			{
				ChangelistState->CustomDeltaChangeIndex++;
				// If we wrap around, skip 0 as this is used to detect that we never sent any custom delta properties
				if (ChangelistState->CustomDeltaChangeIndex == 0)
				{
					++ChangelistState->CustomDeltaChangeIndex;
				}
			}

			SendingRepState->CustomDeltaChangeIndex = ChangelistState->CustomDeltaChangeIndex;
		}
		else
		{
			// also increment changes if we had to skip any properties for conditionals, since another connection may generate data
			if (bHasDirtyCustomDeltaProperties && bSkippedPropertyCondition)
			{
				ChangelistState->CustomDeltaChangeIndex++;
				// If we wrap around, skip 0 as this is used to detect that we never sent any custom delta properties
				if (ChangelistState->CustomDeltaChangeIndex == 0)
				{
					++ChangelistState->CustomDeltaChangeIndex;
				}
			}

			// SendingRepState->CustomDeltaChangeIndex needs to be incremented in all cases just like above,
			// otherwise we cannot skip CustomDeltaProperties for connections on which they do not write
			// if they received a NAK on their last write
			
			// this skipping connection is up to date
			SendingRepState->CustomDeltaChangeIndex = ChangelistState->CustomDeltaChangeIndex;
		}
#endif //WITH_PUSH_MODEL
	}

	if ( Connection->ResendAllDataState != EResendAllDataState::None )
	{
		bDirtyForReplay = false;

		// If we are resending data since open, we don't want to affect the current state of channel/replication, so just send the data, and return
		const bool bWroteImportantData = Writer.GetNumBits() != 0;

		if (bWroteImportantData)
		{
			OwningChannel->WriteContentBlockPayload( Object, Bunch, bHasRepLayout, Writer );
			return true;
		}

		return false;
	}

	// LastUpdateEmpty - this is done before dequeing the multicasted unreliable functions on purpose as they should not prevent
	// an actor channel from going dormant.
	bLastUpdateEmpty = Writer.GetNumBits() == 0;

	// Replicate Queued (unreliable functions)
	if ( RemoteFunctions != nullptr && RemoteFunctions->GetNumBits() > 0 )
	{
		if ( UNLIKELY(GNetRPCDebug == 1) )
		{
			UE_LOGF( LogRepTraffic, Warning,	"      Sending queued RPCs: %ls. Channel[%d] [%.1f bytes]", *Object->GetName(), OwningChannel->ChIndex, RemoteFunctions->GetNumBits() / 8.f );
		}

		Writer.SerializeBits( RemoteFunctions->GetData(), RemoteFunctions->GetNumBits() );

		// Append potential NetToken exports from queued remote functions.
		Writer.NetTokenExportContext.Get()->AppendNetTokensPendingExport(RemoteFunctions->NetTokensPendingExport);		
		RemoteFunctions->NetTokensPendingExport.Reset();

		RemoteFunctions->Reset();
		RemoteFuncInfo.Empty();

		NETWORK_PROFILER(GNetworkProfiler.FlushQueuedRPCs(OwningChannelConnection, Object));
	}

	// See if we wrote something important (anything but the 'end' int below).
	// Note that queued unreliable functions are considered important (WroteImportantData) but not for bLastUpdateEmpty. LastUpdateEmpty
	// is used for dormancy purposes. WroteImportantData is for determining if we should not include a component in replication.
	const bool WroteImportantData = Writer.GetNumBits() != 0;

	if ( WroteImportantData )
	{
		OwningChannel->WriteContentBlockPayload( Object, Bunch, bHasRepLayout, Writer );
	}

	return WroteImportantData;
}

void FObjectReplicator::ForceRefreshUnreliableProperties()
{
	if (GetObject() == nullptr)
	{
		UE_LOGF(LogRep, Verbose, "ForceRefreshUnreliableProperties: Object == nullptr");
		return;
	}

	check(!bOpenAckCalled);

	if (FSendingRepState* SendingRepState = RepState->GetSendingRepState())
	{
		SendingRepState->bOpenAckedCalled = true;
	}

	bOpenAckCalled = true;
}

void FObjectReplicator::PostSendBunch( FPacketIdRange & PacketRange, uint8 bReliable )
{
	LLM_SCOPE_BYTAG(NetObjReplicator);

	const UObject* Object = GetObject();

	if ( Object == nullptr )
	{
		UE_LOGF(LogNet, Verbose, "PostSendBunch: Object == nullptr");
		return;
	}

	check(RepLayout);

	// Don't update retirement records for reliable properties. This is ok to do only if we also pause replication on the channel until the acks have gone through.
	// Also clean up history and/or retirement records if the bunch failed to send
	const bool bSkipRetirementUpdate = OwningChannel->bPausedUntilReliableACK || (PacketRange.First == INDEX_NONE && PacketRange.Last == INDEX_NONE);

	const FRepLayout& LocalRepLayout = *RepLayout;

	if (FSendingRepState* SendingRepState = RepState.IsValid() ? RepState->GetSendingRepState() : nullptr)
	{
		if (!bSkipRetirementUpdate)
		{
			// Don't call if reliable, since the bunch will be resent. We dont want this to end up in the changelist history
			// But is that enough? How does it know to delta against this latest state?

			for (int32 i = SendingRepState->HistoryStart; i < SendingRepState->HistoryEnd; ++i)
			{
				const int32 HistoryIndex = i % FSendingRepState::MAX_CHANGE_HISTORY;

				FRepChangedHistory & HistoryItem = SendingRepState->ChangeHistory[HistoryIndex];

				if (HistoryItem.OutPacketIdRange.First == INDEX_NONE)
				{
					check(HistoryItem.Changed.Num() > 0);
					check(!HistoryItem.Resend);

					HistoryItem.OutPacketIdRange = PacketRange;

					if (!bReliable && !SendingRepState->bOpenAckedCalled)
					{
						SendingRepState->PreOpenAckHistory.Add(HistoryItem);
					}
				}
			}
		}

		// bPausedUntilReliableAck won't have been set until *after* we generated a changelist and tried to send
		// data. Therefore, there may be a changelist already created, and we need to remove it from the history.
		else
		{
			// We won't try to replicate again until this bunch has been acked, so there should only ever possibly
			// be a single unset history item, and that should correspond to the one that is in this bunch.
			// However, PostSendBunch will be called on all subobject replicators when we replicate an Actor, so it's
			// possible that we either didn't send anything, or the last thing we sent wasn't reliable.
			if (SendingRepState->HistoryEnd > SendingRepState->HistoryStart)
			{
				const int32 HistoryIndex = (SendingRepState->HistoryEnd - 1) % FSendingRepState::MAX_CHANGE_HISTORY;
				FRepChangedHistory& HistoryItem = SendingRepState->ChangeHistory[HistoryIndex];
	
				if (!HistoryItem.WasSent())
				{
					HistoryItem.Reset();
					--SendingRepState->HistoryEnd;
				}
			}
		}

		for (FPropertyRetirement& Retirement : SendingRepState->Retirement)
		{
			FPropertyRetirement* Next = Retirement.Next;
			FPropertyRetirement* Prev = &Retirement;

			while (Next != nullptr)
			{
				// This is updating the dynamic properties retirement record that was created above during property replication
				// (we have to wait until we actually send the bunch to know the packetID, which is why we look for .First==INDEX_NONE)
				if (Next->OutPacketIdRange.First == INDEX_NONE)
				{
					if (!bSkipRetirementUpdate)
					{
						Next->OutPacketIdRange = PacketRange;

						// Mark the last time on this retirement slot that a property actually changed
						Retirement.OutPacketIdRange = PacketRange;
					}
					else
					{
						// We need to remove the retirement entry here!
						Prev->Next = Next->Next;
						delete Next;
						Next = Prev;
					}
				}

				Prev = Next;
				Next = Next->Next;
			}

			ValidateRetirementHistory(Retirement, Object);
		}
	}
}

void FObjectReplicator::FRPCPendingLocalCall::CountBytes(FArchive& Ar) const
{
	Buffer.CountBytes(Ar);
	UnmappedGuids.CountBytes(Ar);
}

void FObjectReplicator::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FObjectReplicator::CountBytes");

	// FObjectReplicator has a shared pointer to an FRepLayout, but since it's shared with
	// the UNetDriver, the memory isn't tracked here.

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RepState",
		if (RepState.IsValid())
		{
			const SIZE_T SizeOfRepState = sizeof(FRepState);
			Ar.CountBytes(SizeOfRepState, SizeOfRepState);
			RepState->CountBytes(Ar);
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ReferencedGuids", ReferencedGuids.CountBytes(Ar));

	// ChangelistMgr points to a ReplicationChangelistMgr managed by the UNetDriver, so it's not tracked here

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RemoveFuncInfo",
		RemoteFuncInfo.CountBytes(Ar);
		if (RemoteFunctions)
		{
			RemoteFunctions->CountMemory(Ar);
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingLocalRPCs",
		PendingLocalRPCs.CountBytes(Ar);
		for (const FRPCPendingLocalCall& PendingRPC : PendingLocalRPCs)
		{
			PendingRPC.Buffer.CountBytes(Ar);
			PendingRPC.UnmappedGuids.CountBytes(Ar);
		}
	);
}

void FObjectReplicator::QueueRemoteFunctionBunch( UFunction* Func, FOutBunch &Bunch )
{
	LLM_SCOPE_BYTAG(NetObjReplicator);

	if (Connection == nullptr)
	{
		return;
	}

	// This is a pretty basic throttling method - just don't let same func be called more than
	// twice in one network update period.
	//
	// Long term we want to have priorities and stronger cross channel traffic management that
	// can handle this better
	int32 InfoIdx = INDEX_NONE;
	for (int32 i = 0; i < RemoteFuncInfo.Num(); ++i)
	{
		if (RemoteFuncInfo[i].FuncName == Func->GetFName())
		{
			InfoIdx = i;
			break;
		}
	}

	if (InfoIdx == INDEX_NONE)
	{
		InfoIdx = RemoteFuncInfo.AddUninitialized();
		RemoteFuncInfo[InfoIdx].FuncName = Func->GetFName();
		RemoteFuncInfo[InfoIdx].Calls = 0;
	}
	
	if (++RemoteFuncInfo[InfoIdx].Calls > CVarMaxRPCPerNetUpdate.GetValueOnAnyThread())
	{
		UE_LOGF(LogRep, Verbose, "Too many calls (%d) to RPC %ls within a single netupdate. Skipping. %ls.  LastCallTime: %.2f. CurrentTime: %.2f. LastRelevantTime: %.2f. LastUpdateTime: %.2f ",
			RemoteFuncInfo[InfoIdx].Calls, *Func->GetName(), *GetPathNameSafe(GetObject()), RemoteFuncInfo[InfoIdx].LastCallTimestamp, OwningChannel->Connection->Driver->GetElapsedTime(), OwningChannel->RelevantTime, OwningChannel->LastUpdateTime);

		// The MustBeMappedGuids can just be dropped, because we aren't actually going to send a bunch. If we don't clear it, then we will get warnings when the next channel tries to replicate
		CastChecked<UPackageMapClient>(Connection->PackageMap)->GetMustBeMappedGuidsInLastBunch().Reset();
		return;
	}
	
	RemoteFuncInfo[InfoIdx].LastCallTimestamp = OwningChannel->Connection->Driver->GetElapsedTime();

	if (RemoteFunctions == nullptr)
	{
		RemoteFunctions = new FOutBunch(OwningChannel, 0);
	}

	RemoteFunctions->SerializeBits(Bunch.GetData(), Bunch.GetNumBits());

	// Save potential NetToken exports for export when we actually commit the RemoteFunctions
	RemoteFunctions->NetTokensPendingExport.Append(Bunch.NetTokensPendingExport);
	Bunch.NetTokensPendingExport.Reset();

	if (Connection->PackageMap != nullptr)
	{
		UPackageMapClient* PackageMapClient = CastChecked<UPackageMapClient>(Connection->PackageMap);

		// We need to copy over any info that was obtained on the package map during serialization, and remember it until we actually call SendBunch
		if (PackageMapClient->GetMustBeMappedGuidsInLastBunch().Num())
		{
			OwningChannel->QueuedMustBeMappedGuidsInLastBunch.Append(PackageMapClient->GetMustBeMappedGuidsInLastBunch());
			PackageMapClient->GetMustBeMappedGuidsInLastBunch().Reset();
		}

		if (!Connection->IsInternalAck())
		{
			// Copy over any additionally required bunches
			TArray<FOutBunch*> AdditionalBunches = PackageMapClient->GetAdditionalRequiredBunches(Bunch, EChannelGetAdditionalRequiredBunchesFlags::None);
			OwningChannel->QueuedExportBunches.Append(MoveTemp(AdditionalBunches));
		}
	}
}

bool FObjectReplicator::ReadyForDormancy(bool bSuppressLogs)
{
	if (GetObject() == nullptr)
	{
		UE_LOGF(LogRep, Verbose, "ReadyForDormancy: Object == nullptr");
		return true;		// Technically, we don't want to hold up dormancy, but the owner needs to clean us up, so we warn
	}

	// Can't go dormant until last update produced no new property updates
	if (!bLastUpdateEmpty)
	{
		UE_CLOGF(!bSuppressLogs, LogRepTraffic, Verbose, "    [%d] Not ready for dormancy. bLastUpdateEmpty = false", OwningChannel->ChIndex);

		return false;
	}

	if (FSendingRepState* SendingRepState = RepState.IsValid() ? RepState->GetSendingRepState() : nullptr)
	{
		if (SendingRepState->HistoryStart != SendingRepState->HistoryEnd)
		{
			// We have change lists that haven't been acked
			return false;
		}

		if (SendingRepState->NumNaks > 0)
		{
			return false;
		}

		if (!SendingRepState->bOpenAckedCalled)
		{
			return false;
		}

		if (SendingRepState->PreOpenAckHistory.Num() > 0)
		{
			return false;
		}

		// Can't go dormant if there are unAckd property updates
		for (FPropertyRetirement& Retirement : SendingRepState->Retirement)
		{
			if (Retirement.Next != nullptr)
			{
				UE_CLOGF(!bSuppressLogs, LogRepTraffic, Verbose, "    [%d] Not ready for dormancy. OutAckPacketId: %d First: %d Last: %d ", OwningChannel->ChIndex, OwningChannel->Connection->OutAckPacketId, Retirement.OutPacketIdRange.First, Retirement.OutPacketIdRange.Last);
				return false;
			}
		}
	}

	return true;
}

void FObjectReplicator::StartBecomingDormant()
{
	if ( GetObject() == nullptr )
	{
		UE_LOGF( LogRep, Verbose, "StartBecomingDormant: Object == nullptr" );
		return;
	}

	bLastUpdateEmpty = false; // Ensure we get one more attempt to update properties
}

void FObjectReplicator::CallRepNotifies(bool bSkipIfChannelHasQueuedBunches) const
{
	// This logic is mostly a copy of FRepLayout::CallRepNotifies, and they should be merged.

	UObject* Object = GetObject();

	if (!IsValid(Object))
	{
		return;
	}

	if (Connection && Connection->Driver && Connection->Driver->ShouldSkipRepNotifies())
	{
		return;
	}

	if (bSkipIfChannelHasQueuedBunches && (OwningChannel && OwningChannel->QueuedBunches.Num() > 0))
	{
		return;
	}

	FReceivingRepState* ReceivingRepState = RepState->GetReceivingRepState();
	RepLayout->CallRepNotifies(ReceivingRepState, Object);

	if (IsValid(Object))
	{
		Object->PostRepNotifies();
	}
}

void FObjectReplicator::UpdateUnmappedObjects(bool & bOutHasMoreUnmapped)
{
	UObject* Object = GetObject();
	
	if (!IsValid(Object))
	{
		bOutHasMoreUnmapped = false;
		return;
	}

	if (Connection->GetConnectionState() == USOCK_Closed)
	{
		UE_LOGF(LogNet, Verbose, "FObjectReplicator::UpdateUnmappedObjects: Connection->State == USOCK_Closed");
		return;
	}

	// Since RepNotifies aren't processed while a channel has queued bunches, don't assert in that case.
	FReceivingRepState* ReceivingRepState = RepState->GetReceivingRepState();
	const bool bHasQueuedBunches = OwningChannel && OwningChannel->QueuedBunches.Num() > 0;

	checkf(bHasQueuedBunches || ReceivingRepState->RepNotifies.Num() == 0 || Connection->Driver->ShouldSkipRepNotifies(),
		TEXT("Failed RepState RepNotifies check. Num=%d. Object=%s. Channel QueuedBunches=%d"),
		ReceivingRepState->RepNotifies.Num(), *Object->GetFullName(), OwningChannel ? OwningChannel->QueuedBunches.Num() : 0);

	bool bCalledPreNetReceive = false;
	bool bSomeObjectsWereMapped = false;

	check(RepLayout);

	const uint32 CurrentReplicationFrame = Connection->GetDriver()->ReplicationFrame;

	const FRepLayout& LocalRepLayout = *RepLayout;

	FNetSerializeCB NetSerializeCB(Connection->Driver);

	FNetDeltaSerializeInfo Parms;
	Parms.Object = Object;
	Parms.Connection = Connection;
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	if (Connection)
	{
		UNetDriver* ConnectionDriver = Connection->GetDriver();
		if (ConnectionDriver && ConnectionDriver->GetWorld())
		{
			Parms.bCallFastArrayClientCallbacks = ConnectionDriver->GetWorld()->IsActorTickAndUserCallbacksEnabled();
		}
	}
#endif
	Parms.bInternalAck = Connection->IsInternalAck();
	Parms.Map = Connection->PackageMap;
	Parms.NetSerializeCB = &NetSerializeCB;

	Parms.bUpdateUnmappedObjects = true;

	// Let the rep layout update any unmapped properties
	LocalRepLayout.UpdateUnmappedObjects(ReceivingRepState, Connection->PackageMap, Object, Parms, bCalledPreNetReceive, bSomeObjectsWereMapped, bOutHasMoreUnmapped);

	bSomeObjectsWereMapped |= Parms.bOutSomeObjectsWereMapped;
	bOutHasMoreUnmapped |= Parms.bOutHasMoreUnmapped;
	bCalledPreNetReceive |= Parms.bCalledPreNetReceive;

	if (bCalledPreNetReceive)
	{
		// If we mapped some objects, make sure to call PostNetReceive (some game code will need to think this was actually replicated to work)
		PostNetReceive();

		UpdateGuidToReplicatorMap();
	}

	// Call any rep notifies that need to happen when object pointers change
	// Pass in false to override the check for queued bunches. Otherwise, if the owning channel has queued bunches,
	// the RepNotifies will remain in the list and the check for 0 RepNotifies above will fail next time.
	CallRepNotifies(false);

	UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap);

	if (PackageMapClient && OwningChannel)
	{
		const bool bIsServer = Connection->Driver->IsServer();
		const FClassNetCache* ClassCache = Connection->Driver->NetCache->GetClassNetCache(ObjectClass);

		// Handle pending RPCs, in order
		for (int32 RPCIndex = 0; RPCIndex < PendingLocalRPCs.Num(); RPCIndex++)
		{
			FRPCPendingLocalCall& Pending = PendingLocalRPCs[RPCIndex];
			const FFieldNetCache* FieldCache = ClassCache->GetFromIndex(Pending.RPCFieldIndex);

			FNetBitReader Reader(Connection->PackageMap, Pending.Buffer.GetData(), Pending.NumBits);
			Connection->SetNetVersionsOnArchive(Reader);

			bool bIsGuidPending = false;

			for (const FNetworkGUID& Guid : Pending.UnmappedGuids)
			{
				if (PackageMapClient->IsGUIDPending(Guid))
				{ 
					bIsGuidPending = true;
					break;
				}
			}

			TSet<FNetworkGUID> UnmappedGuids;
			bool bFunctionWasUnmapped = false;
			bool bSuccess = true;
			FString FunctionName = "(Unknown)";
			
			if (FieldCache == nullptr)
			{
				UE_LOGF(LogNet, Warning, "FObjectReplicator::UpdateUnmappedObjects: FieldCache not found. Object: %ls", *Object->GetFullName());
				bSuccess = false;
			}
			else
			{
				// Force execute if none of our RPC guids are pending, even if other guids are. This is more consistent behavior as it is less dependent on unrelated actors
				ESkipRpcBehavior AllowDelay = bIsGuidPending ? ESkipRpcBehavior::SkipIfNotReady : ESkipRpcBehavior::NeverSkip;

				FunctionName = FieldCache->Field.GetName();
				bSuccess = ReceivedRPC(Reader, UnmappedGuids, bFunctionWasUnmapped, Pending.RepFlags, FieldCache, AllowDelay);
			}

			if (!bSuccess)
			{
				if (bIsServer && !Connection->IsInternalAck())
				{
					// Close our connection and abort rpcs as things are invalid
					PendingLocalRPCs.Empty();
					bOutHasMoreUnmapped = false;

					UE_LOGF(LogNet, Error, "FObjectReplicator::UpdateUnmappedObjects: Failed executing delayed RPC %ls on Object %ls, closing connection!", *FunctionName, *Object->GetFullName());

					Connection->Close();
					return;
				}
				else
				{
					UE_LOGF(LogNet, Warning, "FObjectReplicator::UpdateUnmappedObjects: Failed executing delayed RPC %ls on Object %ls, skipping RPC!", *FunctionName, *Object->GetFullName());

					// Skip this RPC, it was marked invalid internally
					PendingLocalRPCs.RemoveAt(RPCIndex);
					RPCIndex--;
				}
			}
			else if (bFunctionWasUnmapped)
			{
				// Still unmapped, update unmapped list
				Pending.UnmappedGuids = UnmappedGuids;
				bOutHasMoreUnmapped = true;
				
				break;
			}
			else
			{
				// Track RPCs delayed multiple frames
				const uint32 DelayedFrames = (CurrentReplicationFrame >= Pending.FrameQueuedAt) ? (CurrentReplicationFrame - Pending.FrameQueuedAt) : 0u;
				if (DelayedFrames > 0)
				{
					UE_LOGF(LogNet, Verbose, "FObjectReplicator::UpdateUnmappedObjects: RPC %ls on Object %ls was finally executed after being delayed for %u frames (~%f ms)",
						*FunctionName, *Object->GetFullName(), DelayedFrames, 
						DelayedFrames*(1000.f / ((GEngine->GetMaxTickRate(0.0f, true) > 0.0f) ? GEngine->GetMaxTickRate(0.0f, true) : 30.f)));
					Connection->TotalDelayedRPCs++;
					Connection->TotalDelayedRPCsFrameCount += DelayedFrames;
				}
				// We executed, remove this one and continue
				PendingLocalRPCs.RemoveAt(RPCIndex);
				RPCIndex--;
			}
		}
	}
}

void FObjectReplicator::EnqueuePendingTransactionalRPCs()
{
	ensureMsgf(AutoRTFM::IsClosed(), TEXT("%hs is meant to be executed inside of a transaction"), __func__);
	if (!ensure(Connection) || !ensure(Connection->Driver) || !ensure(Connection->Driver->NetCache))
	{
		return;
	}

	const bool bIsServer = Connection->Driver->IsServer();
	if (!ensureMsgf(bIsServer, TEXT("%hs: Expected to only be executed on the server"), __func__))
	{
		return;
	}

	UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap);
	if (!PackageMapClient || !OwningChannel)
	{
		return;
	}

	const UObject* Object = GetObject();
	FString ObjectFullName = Object->GetFullName();

	// Handle pending RPCs, in order
	TArray<FRPCPendingLocalCall> ServiceTheseRPCs;
	Swap(PendingTransactionalRPCs, ServiceTheseRPCs);

	TArray<TUniqueFunction<void(void)>> TransactionsToEnqueue;
	TransactionsToEnqueue.Reserve(ServiceTheseRPCs.Num());

	for (const FRPCPendingLocalCall& Pending : ServiceTheseRPCs)
	{
		TransactionsToEnqueue.Emplace([this, Pending, ObjectFullName]()
			{
				const FClassNetCache* ClassCache = Connection->Driver->NetCache->GetClassNetCache(ObjectClass);

				ensureMsgf(Pending.UnmappedGuids.IsEmpty(), TEXT("PendingTransactionalRPCs are never expected to have UnmappedGuids (UnmappedGuids are in PendingLocalRPCs)"));

				bool bSuccess;
				const FFieldNetCache* FieldCache = ClassCache->GetFromIndex(Pending.RPCFieldIndex);			
				if (FieldCache == nullptr)
				{
					UE_LOGF(LogNet, Warning, "%s: FieldCache not found. Object: %ls", __func__, *ObjectFullName);
					bSuccess = false;
				}
				else
				{
					// Help us keep track of this function's execution time
					FName FunctionName = FieldCache->Field.GetFName();
					UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(FunctionName);
					AutoRTFM::PushOnAbortHandler(&Pending, []()
					{
						UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(TEXT("Abort"));
					});

					{
						UE::RemoteExecutor::FSpeculationExecutionScope OptimizationScope;
						FNetBitReader Reader(Connection->PackageMap, Pending.Buffer.GetData(), Pending.NumBits);
						Connection->SetNetVersionsOnArchive(Reader);

						TSet<FNetworkGUID> UnmappedGuids;
						bool bSkipped = false;
						bSuccess = ReceivedRPC(Reader, UnmappedGuids, bSkipped, Pending.RepFlags, FieldCache, ESkipRpcBehavior::NeverSkip);

						ensureMsgf(UnmappedGuids.IsEmpty(), TEXT("%hs: Servers are expected to never have UnmappedGuids"), __func__);
						ensureMsgf(!bSkipped, TEXT("%hs: We should never skip an RPC when requesting NeverSkip"), __func__);
					}

					UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(TEXT(""));
					AutoRTFM::PopOnAbortHandler(&Pending);
				}

				if (!bSuccess)
				{
					FString FunctionName = FieldCache ? FieldCache->Field.GetName() : TEXT("Unknown");

					const bool bDemoMode = Connection->IsInternalAck();
					if (!bDemoMode)
					{
						// Close our connection and abort rpcs as things are invalid
						UE_LOGF(LogNet, Error, "%s: Failed executing delayed RPC %ls on Object %ls, closing connection!", __func__, *FunctionName, *ObjectFullName);

						Connection->Close();
						return;
					}
					else
					{
						UE_LOGF(LogNet, Warning, "%s: Failed executing delayed RPC %ls on Object %ls, skipping RPC!", __func__, *FunctionName, *ObjectFullName);
					}
				}
			});
	}

	if (CVarEnqueueTransactionalRpcs.GetValueOnAnyThread(true))
	{
		UE::RemoteExecutor::EnqueueWorkList(Object->GetFName(), true, MoveTemp(TransactionsToEnqueue));
	}
	else
	{
		for (const TUniqueFunction<void(void)>& RpcFunc : TransactionsToEnqueue)
		{
			RpcFunc();
		}
	}
}

void FObjectReplicator::QueuePropertyRepNotify(
	UObject* Object,
	FProperty * Property,
	const int32 ElementIndex,
	TArray<uint8>& MetaData)
{
	LLM_SCOPE_BYTAG(NetObjReplicator);

	if (!Property->HasAnyPropertyFlags(CPF_RepNotify))
	{
		return;
	}

	FReceivingRepState* ReceivingRepState = RepState.IsValid() ? RepState->GetReceivingRepState() : nullptr;
	if (ensureMsgf(ReceivingRepState, TEXT("FObjectReplicator::QueuePropertyRepNotifiy: No receiving RepState. Object=%s, Property=%s"),
		*GetPathNameSafe(Object), *Property->GetName()))
	{
		//@note: AddUniqueItem() here for static arrays since RepNotify() currently doesn't indicate index,
		//			so reporting the same property multiple times is not useful and wastes CPU
		//			were that changed, this should go back to AddItem() for efficiency
		// @todo UE - not checking if replicated value is changed from old.  Either fix or document, as may get multiple repnotifies of unacked properties.
		ReceivingRepState->RepNotifies.AddUnique(Property);

		UFunction* RepNotifyFunc = Object->FindFunctionChecked(Property->RepNotifyFunc);

		if (RepNotifyFunc->NumParms > 0)
		{
			if (Property->ArrayDim != 1)
			{
				// For static arrays, we build the meta data here, but adding the Element index that was just read into the PropMetaData array.
				UE_LOGF(LogRepTraffic, Verbose, "Property %ls had ArrayDim: %d change", *Property->GetName(), ElementIndex);

				// Property is multi dimensional, keep track of what elements changed
				TArray< uint8 > & PropMetaData = ReceivingRepState->RepNotifyMetaData.FindOrAdd(Property);
				PropMetaData.Add(ElementIndex);
			}
			else if (MetaData.Num() > 0)
			{
				// For other properties (TArrays only now) the MetaData array is build within ::NetSerialize. Just add it to the RepNotifyMetaData map here.

				//UE_LOGF(LogRepTraffic, Verbose, "Property %ls had MetaData: ", *Property->GetName() );
				//for (auto MetaIt = MetaData.CreateIterator(); MetaIt; ++MetaIt)
				//	UE_LOGF(LogRepTraffic, Verbose, "   %d", *MetaIt );

				// Property included some meta data about what was serialized. 
				TArray< uint8 > & PropMetaData = ReceivingRepState->RepNotifyMetaData.FindOrAdd(Property);
				PropMetaData = MetaData;
			}
		}
	}
}

void FObjectReplicator::WritePropertyHeaderAndPayload(
	UObject* Object,
	FProperty*				Property,
	FNetFieldExportGroup* NetFieldExportGroup,
	FNetBitWriter& Bunch,
	FNetBitWriter& Payload) const
{
	// Get class network info cache.
	const FClassNetCache* ClassCache = Connection->Driver->NetCache->GetClassNetCache(ObjectClass);

	check(ClassCache);

	// Get the network friend property index to replicate
	const FFieldNetCache * FieldCache = ClassCache->GetFromField(Property);

	checkSlow(FieldCache);

	// Send property name and optional array index.
	check(FieldCache->FieldNetIndex <= ClassCache->GetMaxIndex());

	// WriteFieldHeaderAndPayload will return the total number of bits written.
	// So, we subtract out the Payload size to get the actual number of header bits.
	const int32 HeaderBits = static_cast<int64>(OwningChannel->WriteFieldHeaderAndPayload(Bunch, ClassCache, FieldCache, NetFieldExportGroup, Payload)) - Payload.GetNumBits();

	NETWORK_PROFILER(GNetworkProfiler.TrackWritePropertyHeader(Property, HeaderBits, nullptr));
}

FScopedActorRoleSwap::FScopedActorRoleSwap(AActor* InActor)
	: Actor(InActor)
{
	const bool bShouldSwapRoles = Actor != nullptr && Actor->GetRemoteRole() == ROLE_Authority;

	if (bShouldSwapRoles)
	{
		Actor->SwapRoles();
	}
	else
	{
		Actor = nullptr;
	}
}

FScopedActorRoleSwap::~FScopedActorRoleSwap()
{
	if (Actor != nullptr)
	{
		Actor->SwapRoles();
	}
}
