// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DataTable.h"
#include "AssetRegistry/AssetData.h"
#include "Internationalization/StabilizeLocalizationKeys.h"
#include "Misc/DataValidation.h"
#include "Misc/PackageName.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "DataTableCSV.h"
#include "DataTableJSON.h"
#include "EditorFramework/AssetImportData.h"
#include "StructUtils/UserDefinedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataTable)

#define LOCTEXT_NAMESPACE "DataTable"

LLM_DEFINE_TAG(DataTable);

namespace
{
#if WITH_EDITORONLY_DATA
	void GatherDataTableForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
	{
		const UDataTable* const DataTable = CastChecked<UDataTable>(Object);

		PropertyLocalizationDataGatherer.GatherLocalizationDataFromObject(DataTable, GatherTextFlags);

		if (DataTable->RowStruct)
		{
			const FString PathToObject = DataTable->GetPathName();
			for (const auto& Pair : DataTable->GetRowMap())
			{
				const FString PathToRow = PathToObject + TEXT(".") + Pair.Key.ToString();
				PropertyLocalizationDataGatherer.GatherLocalizationDataFromStructWithCallbacks(PathToRow, DataTable->RowStruct, Pair.Value, nullptr, GatherTextFlags);
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	UE_CALL_ONCE(UE::GC::RegisterSlowImplementation, &UDataTable::AddReferencedObjects, UE::GC::EAROFlags::ExtraSlow);
}

UDataTable::FScopedDataTableChange::FScopedDataTableChange(UDataTable* InTable)
	: Table(InTable)
{
	UE::TScopeLock Lock(CriticalSection);
	int32& Count = ScopeCount.FindOrAdd(Table);
	RowName = NAME_None;

	++Count;
}

UDataTable::FScopedDataTableChange::FScopedDataTableChange(UDataTable* InTable, FName Row)
	: Table(InTable)
{
	UE::TScopeLock Lock(CriticalSection);
	int32& Count = ScopeCount.FindOrAdd(Table);
	
	//Recording RowName so we can emit HandleDataTableChanged with the name when adding\removing item without nested scopes.
	RowName = Row;
	++Count;
}

UDataTable::FScopedDataTableChange::~FScopedDataTableChange()
{
	UE::TScopeLock Lock(CriticalSection);
	int32& Count = ScopeCount.FindChecked(Table);
	--Count;
	if (Count == 0)
	{
		Table->HandleDataTableChanged(RowName);
		ScopeCount.Remove(Table);
	}
}

TMap<UDataTable*, int32> UDataTable::FScopedDataTableChange::ScopeCount;
FTransactionallySafeCriticalSection UDataTable::FScopedDataTableChange::CriticalSection;

#define DATATABLE_CHANGE_SCOPE()	UDataTable::FScopedDataTableChange ActiveScope(this);

//This macro should only be used for innermost "locks" like adding\removing a single item. 
#define DATATABLE_CHANGE_SCOPE_SINGLE_ROW(Name)	UDataTable::FScopedDataTableChange ActiveScope(this, Name);

UDataTable::UDataTable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIgnoreExtraFields = false;
	bIgnoreMissingFields = false;
	bStripFromClientBuilds = false;
	bPreserveExistingValues = false;

#if WITH_EDITORONLY_DATA
	{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(UDataTable::StaticClass(), &GatherDataTableForLocalization); }
#endif
}

#if WITH_EDITOR
void UDataTable::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

#if WITH_EDITORONLY_DATA
	HandleDataTableChanged();
#endif
}
#endif

void UDataTable::LoadStructData(FStructuredArchiveSlot Slot)
{
	int32 NumRows;
	FStructuredArchiveArray Array = Slot.EnterArray(NumRows);

	UScriptStruct* LoadUsingStruct = RowStruct;
	if (!LoadUsingStruct)
	{
		if (NumRows > 0 && Slot.GetUnderlyingArchive().UseUnversionedPropertySerialization())
		{
			UE_LOGF(LogDataTable, Error, "Failed to load DataTable '%ls' with %d rows. The rows can't be loaded with UnversionedPropertySerialization when the RowStruct is missing.", *GetPathName(), NumRows);
			Slot.GetUnderlyingArchive().SetError();
			return;
		}
		if (!HasAnyFlags(RF_ClassDefaultObject) && GetOutermost() != GetTransientPackage())
		{
			UE_LOGF(LogDataTable, Warning, "Missing RowStruct while loading DataTable '%ls', NeedLoad: '%ls'!", *GetPathName(), HasAnyFlags(RF_NeedLoad) ? TEXT("true") : TEXT("false"));
		}
		LoadUsingStruct = FTableRowBase::StaticStruct();
	}

	DATATABLE_CHANGE_SCOPE();

	RowMap.Reserve(NumRows);
	for (int32 RowIdx = 0; RowIdx < NumRows; RowIdx++)
	{
		FStructuredArchiveRecord RowRecord = Array.EnterElement().EnterRecord();

		// Load row name
		FName RowName;
		RowRecord << SA_VALUE(TEXT("Name"), RowName);

		// Load row data
		uint8* RowData = (uint8*)FMemory::Malloc(LoadUsingStruct->GetStructureSize());

		// And be sure to call DestroyScriptStruct later
		LoadUsingStruct->InitializeStruct(RowData);

		LoadUsingStruct->SerializeItem(RowRecord.EnterField(TEXT("Value")), RowData, nullptr);

		// Add to map
		RowMap.Add(RowName, RowData);
	}
}

void UDataTable::SaveStructData(FStructuredArchiveSlot Slot)
{
	UScriptStruct* SaveUsingStruct = RowStruct;
	if (!SaveUsingStruct)
	{
		if (!HasAnyFlags(RF_ClassDefaultObject) && GetOutermost() != GetTransientPackage())
		{
			UE_LOGF(LogDataTable, Error, "Missing RowStruct while saving DataTable '%ls', NeedLoad: '%ls'!", *GetPathName(), HasAnyFlags(RF_NeedLoad) ? TEXT("true") : TEXT("false"));
		}
		SaveUsingStruct = FTableRowBase::StaticStruct();
	}

	int32 NumRows = RowMap.Num();
	FStructuredArchiveArray Array = Slot.EnterArray(NumRows);

	// Now iterate over rows in the map
	for (auto RowIt = RowMap.CreateIterator(); RowIt; ++RowIt)
	{
		// Save out name
		FName RowName = RowIt.Key();
		FStructuredArchiveRecord Row = Array.EnterElement().EnterRecord();
		Row << SA_VALUE(TEXT("Name"), RowName);

		// Save out data
		uint8* RowData = RowIt.Value();

		SaveUsingStruct->SerializeItem(Row.EnterField(TEXT("Value")), RowData, nullptr);
	}
}

void UDataTable::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	OutDeps.Add(RowStruct);
}

void UDataTable::OnPostDataImported(TArray<FString>& OutCollectedImportProblems)
{
	if (RowStruct)
	{
		const bool bIsNativeRowStruct = RowStruct->IsChildOf(FTableRowBase::StaticStruct());

		FString DataTableTextNamespace = GetName();
#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor)
	{
			DataTableTextNamespace = TextNamespaceUtil::BuildFullNamespace(DataTableTextNamespace, TextNamespaceUtil::EnsurePackageNamespace(this), /*bAlwaysApplyPackageNamespace*/true);
		}
#endif

		for (const TPair<FName, uint8*>& TableRowPair : RowMap)
		{
			if (bIsNativeRowStruct)
			{
				FTableRowBase* CurRow = reinterpret_cast<FTableRowBase*>(TableRowPair.Value);
				CurRow->OnPostDataImport(this, TableRowPair.Key, OutCollectedImportProblems);
			}

#if WITH_EDITOR
			// Perform automatic fix-up on any text properties that have been imported from a raw string to assign them deterministic keys
			// We do this after OnPostDataImport has been run on the row, as that function may perform custom fix-up logic that will fix the keys differently than the default logic
			StabilizeLocalizationKeys::StabilizeLocalizationKeysForStruct(RowStruct, TableRowPair.Value, DataTableTextNamespace, TableRowPair.Key.ToString());
#endif
		}
	}
	
	// Don't need to call HandleDataTableChanged because it gets called by the scope and post edit callbacks
	// If you need to handle an import-specific problem, register with FDataTableEditorUtils
}

void UDataTable::HandleDataTableChanged(FName ChangedRowName)
{
	if (!IsValidChecked(this) || IsUnreachable() || HasAnyFlags(RF_BeginDestroyed))
	{
		// This gets called during destruction, don't broadcast callbacks
		return;
	}

	// Do the row fixup before global callback
	if (RowStruct)
	{
		const bool bIsNativeRowStruct = RowStruct->IsChildOf(FTableRowBase::StaticStruct());

		if (bIsNativeRowStruct)
		{
			//Avoid iterating on the full table when a specific row was modified. 
			if (ChangedRowName != NAME_None)
			{
				FTableRowBase* CurRow = reinterpret_cast<FTableRowBase*>(FindRowUnchecked(ChangedRowName));
				
				if (CurRow)
				{
					CurRow->OnDataTableChanged(this, ChangedRowName);
				}
			}
			else
			{
				for (const TPair<FName, uint8*>& TableRowPair : RowMap)
				{
					FTableRowBase* CurRow = reinterpret_cast<FTableRowBase*>(TableRowPair.Value);
					CurRow->OnDataTableChanged(this, TableRowPair.Key);
				}
			}
		}
	}

	OnDataTableChanged().Broadcast();
}

void UDataTable::Serialize(FStructuredArchiveRecord Record)
{
	FArchive& BaseArchive = Record.GetUnderlyingArchive();
	LLM_SCOPE_BYTAG(DataTable);

#if WITH_EDITORONLY_DATA
	// Make sure and update RowStructName before calling the parent Serialize (which will save the properties)
	if (BaseArchive.IsSaving() && RowStruct)
	{
		RowStructName_DEPRECATED = RowStruct->GetFName();
		RowStructPathName = RowStruct->GetStructPathName();
	}
#endif	// WITH_EDITORONLY_DATA

	Super::Serialize(Record); // When loading, this should load our RowStruct!	

	if (RowStruct)
	{
		RowStruct->ConditionalPreload();
	}

	if(BaseArchive.IsLoading())
	{
		DATATABLE_CHANGE_SCOPE();
		EmptyTable();
		LoadStructData(Record.EnterField(TEXT("Data")));
	}
	else if(BaseArchive.IsSaving())
	{
		SaveStructData(Record.EnterField(TEXT("Data")));
	}
}

void UDataTable::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{	
	UDataTable* This = CastChecked<UDataTable>(InThis);

	// Need to emit references for referenced rows (unless there's no properties that reference UObjects)
	if(This->RowStruct != nullptr && This->RowStruct->RefLink != nullptr)
	{
		// Now iterate over rows in the map
		for (const TPair<FName, uint8*>& Pair : This->RowMap)
		{
			if (uint8* RowData = Pair.Value)
			{
				Collector.AddPropertyReferencesWithStructARO(This->RowStruct, RowData, This);
			}
		}
	}

	Super::AddReferencedObjects( This, Collector );
}

void UDataTable::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(RowMap.GetAllocatedSize());
	if (RowStruct)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(RowMap.Num() * RowStruct->GetStructureSize());
	}
}

void UDataTable::FinishDestroy()
{
	Super::FinishDestroy();
	if(!IsTemplate())
	{
		EmptyTable(); // Free memory when UObject goes away
	}
}

#if WITH_EDITORONLY_DATA

FTopLevelAssetPath UDataTable::GetRowStructPathName() const
{
	return (RowStruct) ? RowStruct->GetStructPathName() : RowStructPathName;
}

void UDataTable::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}

	// Add the row structure tag
	{
		static const FName RowStructureTag = "RowStructure";
		Context.AddTag( FAssetRegistryTag(RowStructureTag, GetRowStructPathName().ToString(), FAssetRegistryTag::TT_Alphabetical) );
	}

	Super::GetAssetRegistryTags(Context);
}

void UDataTable::PostInitProperties()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}

	Super::PostInitProperties();
}

void UDataTable::PostLoad()
{
	Super::PostLoad();
	if (!ImportPath_DEPRECATED.IsEmpty() && AssetImportData)
	{
		FAssetImportInfo Info;
		Info.Insert(FAssetImportInfo::FSourceFile(ImportPath_DEPRECATED));
		AssetImportData->SourceData = MoveTemp(Info);
	}
	if (!RowStructName_DEPRECATED.IsNone())
	{
		UStruct* SavedRowStruct = RowStruct;
		if (!SavedRowStruct)
		{
			SavedRowStruct = FindFirstObjectSafe<UStruct>(*RowStructName_DEPRECATED.ToString());
		}
		if (SavedRowStruct)
		{
			RowStructPathName = SavedRowStruct->GetStructPathName();
		}
		else
		{
			UE_LOGF(LogDataTable, Error, "Unable to resolved RowStruct PathName from serialized short name '%ls'!", *RowStructName_DEPRECATED.ToString());
		}
	}
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void UDataTable::ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const
{
	Super::ThreadedPostLoadAssetRegistryTagsOverride(Context);

	static const FName RowStructureTag("RowStructure");
	FString TagValue = Context.GetAssetData().GetTagValueRef<FString>(RowStructureTag);
	if (!TagValue.IsEmpty() && FPackageName::IsShortPackageName(TagValue))
	{
		FTopLevelAssetPath PathName = UClass::TryConvertShortTypeNameToPathName<UField>(TagValue, ELogVerbosity::Warning, TEXT("UDataTable::ThreadedPostLoadAssetRegistryTagsOverride"));
		if (!PathName.IsNull())
		{
			Context.AddTagToUpdate(FAssetRegistryTag(RowStructureTag, PathName.ToString(), FAssetRegistryTag::TT_Alphabetical));
		}
	}
}

EDataValidationResult UDataTable::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	if (RowStruct && RowStruct->IsChildOf(FTableRowBase::StaticStruct()))
	{
		FDataValidationContext RowContext(Context.WasAssetLoadedForValidation(), Context.GetValidationUsecase(), Context.GetAssociatedExternalObjects(), Context.GetMessageLogName());
		int32 LastNumIssues = 0;
		for (const TPair<FName, uint8*>& TableRowPair : RowMap)
		{
			const FTableRowBase* CurRow = reinterpret_cast<FTableRowBase*>(TableRowPair.Value);
			Result = CombineDataValidationResults(Result, CurRow->IsDataValid(RowContext));

			const TArray<FDataValidationContext::FIssue>& Issues = RowContext.GetIssues();
			const int32 NewNumIssues = Issues.Num();
			for (int32 IssueIdx = LastNumIssues; IssueIdx < NewNumIssues; ++IssueIdx)
			{
				const FDataValidationContext::FIssue& Issue = Issues[IssueIdx];
				if (Issue.TokenizedMessage.IsValid())
				{
					TSharedRef<FTokenizedMessage> TokenizedMessage = Issue.TokenizedMessage->Clone();
					TokenizedMessage->AddText(FText::Format(LOCTEXT("DataTableValidation.RowName", "[Row Name: {0}]"), FText::FromName(TableRowPair.Key)));

					Context.AddMessage(TokenizedMessage);
				}
				else
				{
					const FText Message = FText::Format(LOCTEXT("DataTableValidation.RowNameWithMsg", "{0} [Row Name: {1}]"), Issue.Message, FText::FromName(TableRowPair.Key));
					if (Issue.Severity == EMessageSeverity::Error)
					{
						Context.AddError(Message);
					}
					else
					{
						Context.AddWarning(Message);
					}
				}
			}

			LastNumIssues = NewNumIssues;
		}
	}
	
	return Result;
}
#endif // WITH_EDITOR

UScriptStruct& UDataTable::GetEmptyUsingStruct() const
{
	UScriptStruct* EmptyUsingStruct = RowStruct;
	if (!EmptyUsingStruct)
	{
		if (!HasAnyFlags(RF_ClassDefaultObject) && GetOutermost() != GetTransientPackage())
		{
			UE_LOGF(LogDataTable, Error, "Missing RowStruct while emptying DataTable '%ls', NeedLoad: '%ls'!", *GetPathName(), HasAnyFlags(RF_NeedLoad) ? TEXT("true") : TEXT("false"));
		}
		EmptyUsingStruct = FTableRowBase::StaticStruct();
	}

	return *EmptyUsingStruct;
}

void UDataTable::EmptyTable()
{
	DATATABLE_CHANGE_SCOPE();

	UScriptStruct& EmptyUsingStruct = GetEmptyUsingStruct();

	// Iterate over all rows in table and free mem
	for (auto RowIt = RowMap.CreateIterator(); RowIt; ++RowIt)
	{
		uint8* RowData = RowIt.Value();
		EmptyUsingStruct.DestroyStruct(RowData);
		FMemory::Free(RowData);
	}

	// Finally empty the map
	RowMap.Empty();
}

void UDataTable::RemoveRow(FName RowName)
{
	DATATABLE_CHANGE_SCOPE_SINGLE_ROW(RowName);

	RemoveRowInternal(RowName);
}

void UDataTable::RemoveRowInternal(FName RowName)
{
	UScriptStruct& EmptyUsingStruct = GetEmptyUsingStruct();

	uint8* RowData = nullptr;
	RowMap.RemoveAndCopyValue(RowName, RowData);
		
	if (RowData)
	{
		EmptyUsingStruct.DestroyStruct(RowData);
		FMemory::Free(RowData);
	}
}

void UDataTable::AddRow(FName RowName, const FTableRowBase& RowData)
{
	DATATABLE_CHANGE_SCOPE_SINGLE_ROW(RowName);

	UScriptStruct& EmptyUsingStruct = GetEmptyUsingStruct();

	// We want to delete the row memory even for child classes that override remove
	RemoveRowInternal(RowName);
		
	uint8* NewRawRowData = (uint8*)FMemory::Malloc(EmptyUsingStruct.GetStructureSize());
	
	EmptyUsingStruct.InitializeStruct(NewRawRowData);
	EmptyUsingStruct.CopyScriptStruct(NewRawRowData, &RowData);

	// Add to map
	AddRowInternal(RowName, NewRawRowData);
}

void UDataTable::AddRow(FName RowName, const uint8* RowData, const UScriptStruct* RowType)
{
	DATATABLE_CHANGE_SCOPE_SINGLE_ROW(RowName);

	UScriptStruct& EmptyUsingStruct = GetEmptyUsingStruct();

	checkf(RowType == &EmptyUsingStruct, TEXT("AddRow called with an incompatible row type! Got '%s', but expected '%s'"), *RowType->GetPathName(), *EmptyUsingStruct.GetPathName());

	// We want to delete the row memory even for child classes that override remove
	RemoveRowInternal(RowName);

	uint8* NewRawRowData = (uint8*)FMemory::Malloc(EmptyUsingStruct.GetStructureSize());

	EmptyUsingStruct.InitializeStruct(NewRawRowData);
	EmptyUsingStruct.CopyScriptStruct(NewRawRowData, RowData);

	// Add to map
	AddRowInternal(RowName, NewRawRowData);
}

void UDataTable::AddRowInternal(FName RowName, uint8* RowData)
{
	RowMap.Add(RowName, RowData);
}

/** Returns the column property where PropertyName matches the name of the column property. Returns NULL if no match is found or the match is not a supported table property */
FProperty* UDataTable::FindTableProperty(const FName& PropertyName) const
{
	FProperty* Property = nullptr;

	if (RowStruct)
	{
		Property = RowStruct->FindPropertyByName(PropertyName);
		if (Property == nullptr && RowStruct->IsA<UUserDefinedStruct>())
		{
			const FString PropertyNameStr = PropertyName.ToString();

			for (TFieldIterator<FProperty> It(RowStruct); It; ++It)
			{
				if (PropertyNameStr == RowStruct->GetAuthoredNameForField(*It))
				{
					Property = *It;
					break;
				}
			}
		}
		if (!DataTableUtils::IsSupportedTableProperty(Property))
		{
			Property = nullptr;
		}
	}

	return Property;
}

#if WITH_EDITOR

void UDataTable::CleanBeforeStructChange()
{
	if (RowsSerializedWithTags.Num() > 0)
	{
		// This is part of an undo, so restore that value instead of calculating a new one
		EmptyTable();
	}
	else
	{
		RowsSerializedWithTags.Reset();
		TemporarilyReferencedObjects.Empty();
		{
			class FRawStructWriter : public FObjectWriter
			{
				TSet<TObjectPtr<UObject>>& TemporarilyReferencedObjects;
			public:
				FRawStructWriter(TArray<uint8>& InBytes, TSet<TObjectPtr<UObject>>& InTemporarilyReferencedObjects)
					: FObjectWriter(InBytes), TemporarilyReferencedObjects(InTemporarilyReferencedObjects) {}
				virtual FArchive& operator<<(class UObject*& Res) override
				{
					FObjectWriter::operator<<(Res);
					TemporarilyReferencedObjects.Add(Res);
					return *this;
				}
			};

			FRawStructWriter MemoryWriter(RowsSerializedWithTags, TemporarilyReferencedObjects);
			SaveStructData(FStructuredArchiveFromArchive(MemoryWriter).GetSlot());
		}

		EmptyTable();
		Modify();
	}
}

void UDataTable::RestoreAfterStructChange()
{
	DATATABLE_CHANGE_SCOPE();

	EmptyTable();
	{
		class FRawStructReader : public FObjectReader
		{
		public:
			FRawStructReader(TArray<uint8>& InBytes) : FObjectReader(InBytes) {}
			virtual FArchive& operator<<(class UObject*& Res) override
			{
				UObject* Object = nullptr;
				FObjectReader::operator<<(Object);
				FWeakObjectPtr WeakObjectPtr = Object;
				Res = WeakObjectPtr.Get();
				return *this;
			}
		};

		FRawStructReader MemoryReader(RowsSerializedWithTags);
		LoadStructData(FStructuredArchiveFromArchive(MemoryReader).GetSlot());
	}
	TemporarilyReferencedObjects.Empty();
	RowsSerializedWithTags.Empty();
}

FString UDataTable::GetTableAsString(const EDataTableExportFlags InDTExportFlags) const
{
	FString Result;

	if (RowStruct != nullptr)
	{
		Result += FString::Printf(TEXT("Using RowStruct: %s\n\n"), *RowStruct->GetPathName());

		// First build array of properties
		TArray<FProperty*> StructProps;
		for (TFieldIterator<FProperty> It(RowStruct); It; ++It)
		{
			FProperty* Prop = *It;
			check(Prop != nullptr);
			StructProps.Add(Prop);
		}

		// First row, column titles, taken from properties
		Result += TEXT("---");
		for (int32 PropIdx=0; PropIdx<StructProps.Num(); PropIdx++)
		{
			Result += TEXT(",");
			Result += StructProps[PropIdx]->GetName();
		}
		Result += TEXT("\n");

		// Now iterate over rows
		for ( auto RowIt = RowMap.CreateConstIterator(); RowIt; ++RowIt )
		{
			FName RowName = RowIt.Key();
			Result += RowName.ToString();

			uint8* RowData = RowIt.Value();
			for(int32 PropIdx=0; PropIdx<StructProps.Num(); PropIdx++)
			{
				Result += TEXT(",");
				Result += DataTableUtils::GetPropertyValueAsString(StructProps[PropIdx], RowData, InDTExportFlags);
			}
			Result += TEXT("\n");			
		}
	}
	else
	{
		Result += FString("Missing RowStruct!\n");
	}
	return Result;
}

FString UDataTable::GetTableAsCSV(const EDataTableExportFlags InDTExportFlags) const
{
	FString Result;
	if (!FDataTableExporterCSV(InDTExportFlags, Result).WriteTable(*this))
	{
		Result = TEXT("Missing RowStruct!\n");
	}
	return Result;
}

FString UDataTable::GetTableAsJSON(const EDataTableExportFlags InDTExportFlags) const
{
	FString Result;
	if (!FDataTableExporterJSON(InDTExportFlags, Result).WriteTable(*this))
	{
		Result = TEXT("Missing RowStruct!\n");
	}
	return Result;
}

template<typename CharType>
bool UDataTable::WriteRowAsJSON(const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter, const void* RowData, const EDataTableExportFlags InDTExportFlags) const
{
	return TDataTableExporterJSON<CharType>(InDTExportFlags, JsonWriter).WriteRow(RowStruct, RowData);
}

template ENGINE_API bool UDataTable::WriteRowAsJSON<TCHAR>(const TSharedRef< TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR> > >& JsonWriter, const void* RowData, const EDataTableExportFlags InDTExportFlags) const;
template ENGINE_API bool UDataTable::WriteRowAsJSON<ANSICHAR>(const TSharedRef< TJsonWriter<ANSICHAR, TPrettyJsonPrintPolicy<ANSICHAR> > >& JsonWriter, const void* RowData, const EDataTableExportFlags InDTExportFlags) const;

bool UDataTable::CopyImportOptions(UDataTable* SourceTable)
{
	// Only safe to call on an empty table
	if (!SourceTable || !ensure(RowMap.Num() == 0))
	{
		return false;
	}

	bStripFromClientBuilds = SourceTable->bStripFromClientBuilds;
	bIgnoreExtraFields = SourceTable->bIgnoreExtraFields;
	bIgnoreMissingFields = SourceTable->bIgnoreMissingFields;
	bPreserveExistingValues = SourceTable->bPreserveExistingValues;
	ImportKeyField = SourceTable->ImportKeyField;
	RowStruct = SourceTable->RowStruct;

	if (RowStruct)
	{
		RowStructName_DEPRECATED = RowStruct->GetFName();
		RowStructPathName = RowStruct->GetStructPathName();
	}

	if (SourceTable->AssetImportData)
	{
		AssetImportData->SourceData = SourceTable->AssetImportData->SourceData;
	}

	return true;
}

template<typename CharType>
bool UDataTable::WriteTableAsJSON(const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter, const EDataTableExportFlags InDTExportFlags) const
{
	return TDataTableExporterJSON<CharType>(InDTExportFlags, JsonWriter).WriteTable(*this);
}

template ENGINE_API bool UDataTable::WriteTableAsJSON<TCHAR>(const TSharedRef< TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR> > >& JsonWriter, const EDataTableExportFlags InDTExportFlags) const;
template ENGINE_API bool UDataTable::WriteTableAsJSON<ANSICHAR>(const TSharedRef< TJsonWriter<ANSICHAR, TPrettyJsonPrintPolicy<ANSICHAR> > >& JsonWriter, const EDataTableExportFlags InDTExportFlags) const;

template<typename CharType>
bool UDataTable::WriteTableAsJSONObject(const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter, const EDataTableExportFlags InDTExportFlags) const
{
	return TDataTableExporterJSON<CharType>(InDTExportFlags, JsonWriter).WriteTableAsObject(*this);
}

template ENGINE_API bool UDataTable::WriteTableAsJSONObject<TCHAR>(const TSharedRef< TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR> > >& JsonWriter, const EDataTableExportFlags InDTExportFlags) const;
template ENGINE_API bool UDataTable::WriteTableAsJSONObject<ANSICHAR>(const TSharedRef< TJsonWriter<ANSICHAR, TPrettyJsonPrintPolicy<ANSICHAR> > >& JsonWriter, const EDataTableExportFlags InDTExportFlags) const;
#endif

TArray<FProperty*> UDataTable::GetTablePropertyArray(const TArray<const TCHAR*>& Cells, UStruct* InRowStruct, TArray<FString>& OutProblems, int32 KeyColumn)
{
	TArray<FProperty*> ColumnProps;

	// Get list of all expected properties from the struct
	TArray<FName> ExpectedPropNames = DataTableUtils::GetStructPropertyNames(InRowStruct);

	// Need at least 2 columns, first column will contain row names
	if(Cells.Num() > 1)
	{
		ColumnProps.AddZeroed( Cells.Num() );

		// Skip first column depending on option
		TArray<FString> TempPropertyImportNames;
		for (int32 ColIdx = 0; ColIdx < Cells.Num(); ++ColIdx)
		{
			if (ColIdx == KeyColumn)
			{
				continue;
			}

			const TCHAR* ColumnValue = Cells[ColIdx];

			FName PropName = DataTableUtils::MakeValidName(ColumnValue);
			if(PropName == NAME_None)
			{
				OutProblems.Add(FString::Printf(TEXT("Missing name for column %d."), ColIdx));
			}
			else
			{
				FProperty* ColumnProp = FindFProperty<FProperty>(InRowStruct, PropName);

				for (TFieldIterator<FProperty> It(InRowStruct); It && !ColumnProp; ++It)
				{
					DataTableUtils::GetPropertyImportNames(*It, TempPropertyImportNames);
					ColumnProp = TempPropertyImportNames.Contains(ColumnValue) ? *It : nullptr;
				}

				// Didn't find a property with this name, problem..
				if(ColumnProp == nullptr)
				{
					if (!bIgnoreExtraFields)
					{
						OutProblems.Add(FString::Printf(TEXT("Cannot find Property for column '%s' in struct '%s'."), *PropName.ToString(), *InRowStruct->GetName()));
					}
				}
				// Found one!
				else
				{
					// Check we don't have this property already
					if(ColumnProps.Contains(ColumnProp))
					{
						OutProblems.Add(FString::Printf(TEXT("Duplicate column '%s'."), *ColumnProp->GetName()));
					}
					// Check we support this property type
					else if( !DataTableUtils::IsSupportedTableProperty(ColumnProp) )
					{
						OutProblems.Add(FString::Printf(TEXT("Unsupported Property type for struct member '%s'."), *ColumnProp->GetName()));
					}
					// Looks good, add to array
					else
					{
						ColumnProps[ColIdx] = ColumnProp;
					}

					// Track that we found this one
					ExpectedPropNames.Remove(ColumnProp->GetFName());
				}
			}
		}
	}

	if (!bIgnoreMissingFields)
	{
		// Generate warning for any properties in struct we are not filling in
		for (int32 PropIdx = 0; PropIdx < ExpectedPropNames.Num(); PropIdx++)
		{
			const FProperty* const ColumnProp = FindFProperty<FProperty>(InRowStruct, ExpectedPropNames[PropIdx]);

#if WITH_EDITOR
			// If the structure has specified the property as optional for import (gameplay code likely doing a custom fix-up or parse of that property),
			// then avoid warning about it
			static const FName DataTableImportOptionalMetadataKey("DataTableImportOptional");
			if (ColumnProp->HasMetaData(DataTableImportOptionalMetadataKey))
			{
				continue;
			}
#endif // WITH_EDITOR

			const FString DisplayName = DataTableUtils::GetPropertyExportName(ColumnProp);
			OutProblems.Add(FString::Printf(TEXT("Expected column '%s' not found in input."), *DisplayName));
		}
	}

	return ColumnProps;
}

TArray<FString> UDataTable::CreateTableFromCSVString(const FString& InString)
{
	DATATABLE_CHANGE_SCOPE();

	// Array used to store problems about table creation
	TArray<FString> OutProblems;

	FDataTableImporterCSV(*this, InString, OutProblems).ReadTable();
	OnPostDataImported(OutProblems);

	return OutProblems;
}

TArray<FString> UDataTable::CreateTableFromJSONString(const FString& InString)
{
	DATATABLE_CHANGE_SCOPE();

	// Array used to store problems about table creation
	TArray<FString> OutProblems;

	FDataTableImporterJSON(*this, InString, OutProblems).ReadTable();
	OnPostDataImported(OutProblems);

	return OutProblems;
}

TArray<FString> UDataTable::CreateTableFromOtherTable(const UDataTable* InTable)
{
	DATATABLE_CHANGE_SCOPE();

	// Array used to store problems about table creation
	TArray<FString> OutProblems;

	if (InTable == nullptr)
	{
		OutProblems.Add(TEXT("No input table provided"));
		return OutProblems;
	}

	if (RowStruct && RowMap.Num() > 0)
	{
		EmptyTable();
	}

	RowStruct = InTable->RowStruct;

	// make a local copy of the rowmap so we have a snapshot of it
	TMap<FName, uint8*> InRowMapCopy = InTable->GetRowMap();

	UScriptStruct& EmptyUsingStruct = GetEmptyUsingStruct();
	for (TMap<FName, uint8*>::TConstIterator RowMapIter(InRowMapCopy.CreateConstIterator()); RowMapIter; ++RowMapIter)
	{
		uint8* NewRawRowData = (uint8*)FMemory::Malloc(EmptyUsingStruct.GetStructureSize());
		EmptyUsingStruct.InitializeStruct(NewRawRowData);
		EmptyUsingStruct.CopyScriptStruct(NewRawRowData, RowMapIter.Value());
		RowMap.Add(RowMapIter.Key(), NewRawRowData);
	}

	return OutProblems;
}

TArray<FString> UDataTable::CreateTableFromRawData(TMap<FName, const uint8*>& DataMap, UScriptStruct* InRowStruct)
{
	DATATABLE_CHANGE_SCOPE();

	// Array used to store problems about table creation
	TArray<FString> OutProblems;

	if (InRowStruct == nullptr)
	{
		OutProblems.Add(TEXT("No input struct provided"));
		return OutProblems;
	}

	if (RowStruct && RowMap.Num() > 0)
	{
		EmptyTable();
	}

	RowStruct = InRowStruct;

	UScriptStruct& EmptyUsingStruct = GetEmptyUsingStruct();
	for (TMap<FName, const uint8*>::TConstIterator RowMapIter(DataMap.CreateConstIterator()); RowMapIter; ++RowMapIter)
	{
		uint8* NewRawRowData = static_cast<uint8*>(FMemory::Malloc(EmptyUsingStruct.GetStructureSize()));
		EmptyUsingStruct.InitializeStruct(NewRawRowData);
		EmptyUsingStruct.CopyScriptStruct(NewRawRowData, RowMapIter.Value());
		RowMap.Add(RowMapIter.Key(), NewRawRowData);
	}

	return OutProblems;
}

#if WITH_EDITOR

TArray<FString> UDataTable::GetColumnTitles() const
{
	TArray<FString> Result;
	Result.Add(TEXT("Name"));
	if (RowStruct)
	{
		for (TFieldIterator<FProperty> It(RowStruct); It; ++It)
		{
			FProperty* Prop = *It;
			check(Prop != nullptr);
			const FString DisplayName = DataTableUtils::GetPropertyExportName(Prop);
			Result.Add(DisplayName);
		}
	}
	return Result;
}

TArray<FString> UDataTable::GetUniqueColumnTitles() const
{
	TArray<FString> Result;
	Result.Add(TEXT("Name"));
	if (RowStruct)
	{
		for (TFieldIterator<FProperty> It(RowStruct); It; ++It)
		{
			FProperty* Prop = *It;
			check(Prop != nullptr);
			const FString DisplayName = Prop->GetName();
			Result.Add(DisplayName);
		}
	}
	return Result;
}

TArray< TArray<FString> > UDataTable::GetTableData(const EDataTableExportFlags InDTExportFlags) const
{
	 TArray< TArray<FString> > Result;

	 Result.Add(GetColumnTitles());

	 // First build array of properties
	 TArray<FProperty*> StructProps;
	 if (RowStruct)
	 {
	 	for (TFieldIterator<FProperty> It(RowStruct); It; ++It)
	 	{
		 	FProperty* Prop = *It;
			check(Prop != nullptr);
		 	StructProps.Add(Prop);
	 	}
	 }

	 // Now iterate over rows
	 for ( auto RowIt = RowMap.CreateConstIterator(); RowIt; ++RowIt )
	 {
		 TArray<FString> RowResult;
		 FName RowName = RowIt.Key();
		 RowResult.Add(RowName.ToString());

		 uint8* RowData = RowIt.Value();
		 for(int32 PropIdx=0; PropIdx<StructProps.Num(); PropIdx++)
		 {
			 RowResult.Add(DataTableUtils::GetPropertyValueAsString(StructProps[PropIdx], RowData, InDTExportFlags));
		 }
		 Result.Add(RowResult);
	 }
	 return Result;

}

#endif //WITH_EDITOR

TArray<FName> UDataTable::GetRowNames() const
{
	TArray<FName> Keys;
	RowMap.GetKeys(Keys);
	return Keys;
}

bool UDataTable::CommonTypeCheck(UScriptStruct* Type, const TCHAR* MethodName, const TCHAR* ContextString) const
{
	if (RowStruct == nullptr)
	{
		UE_LOGF(LogDataTable, Error, "%ls : DataTable '%ls' has no RowStruct specified (%ls).", MethodName, *GetPathName(), ContextString);
		return false;
	}
	else if (!RowStruct->IsChildOf(Type))
	{
		UE_LOGF(LogDataTable, Error, "%ls : Incorrect type specified for DataTable '%ls' (%ls).", MethodName, *GetPathName(), ContextString);
		return false;
	}

	return true;
}

uint8* UDataTable::FindRowInternal(UScriptStruct* Type, FName RowName, const TCHAR* ContextString, bool bWarnIfRowMissing) const
{
	// NOTE: not quite the same as CommonTypeCheck - if the IsChildOf() test below fails, only warn if bWarnIfRowMissing.
	if(RowStruct == nullptr)
	{
		UE_LOGF(LogDataTable, Error, "UDataTable::FindRow : '%ls' specified no row for DataTable '%ls'.", ContextString, *GetPathName());
		return nullptr;
	}

	if(!RowStruct->IsChildOf(Type))
	{
		UE_CLOGF(bWarnIfRowMissing, LogDataTable, Error, "UDataTable::FindRow : '%ls' specified incorrect type for DataTable '%ls'.", ContextString, *GetPathName());
		return nullptr;
	}

	if(RowName.IsNone())
	{
		UE_CLOGF(bWarnIfRowMissing, LogDataTable, Warning, "UDataTable::FindRow : '%ls' requested invalid row 'None' from DataTable '%ls'.", ContextString, *GetPathName());
		return nullptr;
	}

	uint8* const* RowDataPtr = GetRowMap().Find(RowName);
	if (RowDataPtr == nullptr)
	{
		if (bWarnIfRowMissing)
		{
			UE_LOGF(LogDataTable, Warning, "UDataTable::FindRow : '%ls' requested row '%ls' not in DataTable '%ls'.", ContextString, *RowName.ToString(), *GetPathName());
		}
		return nullptr;
	}

	uint8* RowData = *RowDataPtr;
	check(RowData);

	return RowData;
}

bool FDataTableRowHandle::operator==(FDataTableRowHandle const& Other) const
{
	return DataTable == Other.DataTable && RowName == Other.RowName;
}

bool FDataTableRowHandle::operator != (FDataTableRowHandle const& Other) const
{
	return DataTable != Other.DataTable || RowName != Other.RowName;
}

void FDataTableRowHandle::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsSaving() && !IsNull() && DataTable)
	{
		// Note which row we are pointing to for later searching
		Ar.MarkSearchableName(DataTable, RowName);
	}
}

bool FDataTableCategoryHandle::operator==(FDataTableCategoryHandle const& Other) const
{
	return DataTable == Other.DataTable && ColumnName == Other.ColumnName && RowContents == Other.RowContents;
}

bool FDataTableCategoryHandle::operator != (FDataTableCategoryHandle const& Other) const
{
	return DataTable != Other.DataTable || ColumnName != Other.ColumnName || RowContents != Other.RowContents;
}

#undef LOCTEXT_NAMESPACE
