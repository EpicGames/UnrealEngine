// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableCSV.h"
#include "Engine/DataTable.h"
#include "Serialization/Csv/CsvParser.h"

#if WITH_EDITOR

FDataTableExporterCSV::FDataTableExporterCSV(const EDataTableExportFlags InDTExportFlags, FString& OutExportText)
	: DTExportFlags(InDTExportFlags)
	, ExportedText(OutExportText)
{
}

FDataTableExporterCSV::~FDataTableExporterCSV()
{
}

bool FDataTableExporterCSV::WriteTable(const UDataTable& InDataTable)
{
	if (!InDataTable.RowStruct)
	{
		return false;
	}

	// Write the header (column titles)
	FString ImportKeyField;
	if (!InDataTable.ImportKeyField.IsEmpty())
	{
		// Write actual name if we have it
		ImportKeyField = InDataTable.ImportKeyField;
		ExportedText += ImportKeyField;
	}
	else
	{
	ExportedText += TEXT("---");
	}

	FProperty* SkipProperty = nullptr;
	for (TFieldIterator<FProperty> It(InDataTable.RowStruct); It; ++It)
	{
		FProperty* BaseProp = *It;
		check(BaseProp);

		FString ColumnHeader = DataTableUtils::GetPropertyExportName(BaseProp, DTExportFlags);
		
		if (ColumnHeader == ImportKeyField)
		{
			// Don't write header again if this is the name field, and save for skipping later
			SkipProperty = BaseProp;
			continue;
		}
		
		ExportedText += TEXT(",");
		ExportedText += ColumnHeader;
	}
	ExportedText += TEXT("\n");

	// Write each row
	for (auto RowIt = InDataTable.GetRowMap().CreateConstIterator(); RowIt; ++RowIt)
	{
		FName RowName = RowIt.Key();
		ExportedText += RowName.ToString();

		uint8* RowData = RowIt.Value();
		WriteRow(InDataTable.RowStruct, RowData, SkipProperty);

		ExportedText += TEXT("\n");
	}

	return true;
}

bool FDataTableExporterCSV::WriteRow(const UScriptStruct* InRowStruct, const void* InRowData, const FProperty* SkipProperty)
{
	if (!InRowStruct)
	{
		return false;
	}

	for (TFieldIterator<FProperty> It(InRowStruct); It; ++It)
	{
		FProperty* BaseProp = *It;
		check(BaseProp);

		if (BaseProp == SkipProperty)
		{
			continue;
		}

		const void* Data = BaseProp->ContainerPtrToValuePtr<void>(InRowData, 0);
		WriteStructEntry(InRowData, BaseProp, Data);
	}

	return true;
}

bool FDataTableExporterCSV::WriteStructEntry(const void* InRowData, FProperty* InProperty, const void* InPropertyData)
{
	ExportedText += TEXT(",");

	const FString PropertyValue = DataTableUtils::GetPropertyValueAsString(InProperty, (uint8*)InRowData, DTExportFlags);
	ExportedText += TEXT("\"");
	ExportedText += PropertyValue.Replace(TEXT("\""), TEXT("\"\""));
	ExportedText += TEXT("\"");

	return true;
}


#endif // WITH_EDITOR


FDataTableImporterCSV::FDataTableImporterCSV(UDataTable& InDataTable, FString InCSVData, TArray<FString>& OutProblems)
	: DataTable(&InDataTable)
	, CSVData(MoveTemp(InCSVData))
	, ImportProblems(OutProblems)
{
}

FDataTableImporterCSV::~FDataTableImporterCSV()
{
}

bool FDataTableImporterCSV::ReadTable()
{
	if (CSVData.IsEmpty())
	{
		ImportProblems.Add(TEXT("Input data is empty."));
		return false;
	}

	// Check we have a RowStruct specified
	if (!DataTable->RowStruct)
	{
		ImportProblems.Add(TEXT("No RowStruct specified."));
		return false;
	}

	const FCsvParser Parser(CSVData);
	const auto& Rows = Parser.GetRows();

	// Must have at least a header row (column names)
	if(Rows.Num() < 1)
	{
		ImportProblems.Add(TEXT("Too few rows (there must be at least a header row)."));
		return false;
	}

	// Find property for each column
	int32 KeyColumn = 0;
	if (!DataTable->ImportKeyField.IsEmpty())
	{
		// Search for key column by name
		for (int32 ColIdx = 0; ColIdx < Rows[0].Num(); ++ColIdx)
		{
			const TCHAR* ColumnValue = Rows[0][ColIdx];
			if (DataTable->ImportKeyField == FString(ColumnValue))
			{
				KeyColumn = ColIdx;
				break;
			}
		}
	}
	
	TArray<FProperty*> ColumnProps = DataTable->GetTablePropertyArray(Rows[0], DataTable->RowStruct, ImportProblems, KeyColumn);

	// Empty existing data if don't want to keep existing values
	if (!DataTable->bPreserveExistingValues)
	{
		DataTable->EmptyTable();
	}

	TSet<FName> NewlyAddedRows;
	
	// Iterate over rows
	for(int32 RowIdx=1; RowIdx<Rows.Num(); RowIdx++)
	{
		const TArray<const TCHAR*>& Cells = Rows[RowIdx];

		// Need at least the key column
		if (Cells.Num() <= KeyColumn)
		{
			ImportProblems.Add(FString::Printf(TEXT("Row '%d' has too few cells."), RowIdx));
			continue;
		}

		// Need enough columns in the properties!
		if( ColumnProps.Num() < Cells.Num() )
		{
			ImportProblems.Add(FString::Printf(TEXT("Row '%d' has more cells than properties, is there a malformed string?"), RowIdx));
			continue;
		}

		// Get row name
		FName RowName = DataTableUtils::MakeValidName(Cells[KeyColumn]);

		// Check its not 'none'
		if(RowName == NAME_None)
		{
			if (!DataTable->ImportKeyField.IsEmpty())
			{
				ImportProblems.Add(FString::Printf(TEXT("Row '%d' missing key field '%s'."), RowIdx, *DataTable->ImportKeyField));
			}
			else
			{
				ImportProblems.Add(FString::Printf(TEXT("Row '%d' missing a name."), RowIdx));
			}

			continue;
		}
		
		uint8* ExistingRowData = DataTable->bPreserveExistingValues ? DataTable->FindRowUnchecked(RowName) : nullptr;
		
		if (!DataTable->AllowDuplicateRowsOnImport())
		{
			bool bRowNameAlreadyAdded;
			NewlyAddedRows.Add(RowName, &bRowNameAlreadyAdded);
			if (bRowNameAlreadyAdded)
			{
				ImportProblems.Add(FString::Printf(TEXT("Duplicate row name '%s'."), *RowName.ToString()));
				continue;
			}
		}
		
		// Allocate data to store information, using UScriptStruct to know its size
		uint8* RowData = ExistingRowData ? ExistingRowData : (uint8*)FMemory::Malloc(DataTable->RowStruct->GetStructureSize());
		
		// Add to row map if we don't already have existing row because we are preserve existing values
		if (!ExistingRowData)
		{
			DataTable->RowStruct->InitializeStruct(RowData);
			DataTable->AddRowInternal(RowName, RowData);
		}
		
		// Now iterate over cells (skipping first cell unless we had an explicit name)
		for (int32 CellIdx = 0; CellIdx < Cells.Num(); CellIdx++)
		{
			if (CellIdx == KeyColumn)
			{
				continue;
			}

			// Try and assign string to data using the column property
			FProperty* ColumnProp = ColumnProps[CellIdx];
			const FString CellValue = Cells[CellIdx];

			// If we have no value in the csv for this specific entry then do nothing
			if (DataTable->bPreserveExistingValues && CellValue.IsEmpty())
			{
				continue;
			}
			
			FString Error = DataTableUtils::AssignStringToProperty(CellValue, ColumnProp, RowData);

			// If we failed, output a problem string
			if(Error.Len() > 0)
			{
				FString ColumnName = (ColumnProp != nullptr) 
					? DataTableUtils::GetPropertyExportName(ColumnProp)
					: FString("NONE");
				ImportProblems.Add(FString::Printf(TEXT("Problem assigning string '%s' to property '%s' on row '%s' : %s"), *CellValue, *ColumnName, *RowName.ToString(), *Error));
			}
		}

		// Problem if we didn't have enough cells on this row
		if(Cells.Num() < ColumnProps.Num())
		{
			ImportProblems.Add(FString::Printf(TEXT("Too few cells on row '%s'."), *RowName.ToString()));			
		}
	}

	DataTable->Modify(true);

	return true;
}

