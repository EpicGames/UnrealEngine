// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDataTable;
enum class EDataTableExportFlags : uint8;

#if WITH_EDITOR

class FDataTableExporterCSV
{
public:
	FDataTableExporterCSV(const EDataTableExportFlags InDTExportFlags, FString& OutExportText);

	~FDataTableExporterCSV();

	bool WriteTable(const UDataTable& InDataTable);

	bool WriteRow(const UScriptStruct* InRowStruct, const void* InRowData, const FProperty* SkipProperty = nullptr);

private:
	bool WriteStructEntry(const void* InRowData, FProperty* InProperty, const void* InPropertyData);

	EDataTableExportFlags DTExportFlags;
	FString& ExportedText;
};

#endif // WITH_EDITOR


class FDataTableImporterCSV
{
public:
	FDataTableImporterCSV(UDataTable& InDataTable, FString InCSVData, TArray<FString>& OutProblems);

	~FDataTableImporterCSV();

	bool ReadTable();

private:
	UDataTable* DataTable;
	FString CSVData;
	TArray<FString>& ImportProblems;
};

