// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/DataTableFunctionLibrary.h"
#include "Engine/CurveTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataTableFunctionLibrary)

#if WITH_EDITOR
#include "EditorFramework/AssetImportData.h"
#include "Factories/CSVImportFactory.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#endif //WITH_EDITOR

UDataTableFunctionLibrary::UDataTableFunctionLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UDataTableFunctionLibrary::EvaluateCurveTableRow(UCurveTable* CurveTable, FName RowName, float InXY, TEnumAsByte<EEvaluateCurveTableResult::Type>& OutResult, float& OutXY,const FString& ContextString)
{
	FCurveTableRowHandle Handle;
	Handle.CurveTable = CurveTable;
	Handle.RowName = RowName;
	
	bool found = Handle.Eval(InXY, &OutXY,ContextString);
	
	if (found)
	{
	    OutResult = EEvaluateCurveTableResult::RowFound;
	}
	else
	{
	    OutResult = EEvaluateCurveTableResult::RowNotFound;
	}
}

UFUNCTION(BlueprintCallable, Category = "DataTable")
FString UDataTableFunctionLibrary::CurveTableToJson(UCurveTable* CurveTable)
{
	if (!CurveTable)
	{
		return FString(TEXT(""));
	}
	return CurveTable->GetTableAsJSON();
}

TArray<FString> UDataTableFunctionLibrary::JsonToCurveTable(UCurveTable* CurveTable, const FString& InString)
{
	if (!CurveTable)
	{
		return TArray<FString>({ FString(TEXT("CurveTable is null")) });
	}
	return CurveTable->CreateTableFromJSONString(InString);
}

bool UDataTableFunctionLibrary::AddSimpleCurveToTable(UCurveTable* CurveTable, FName RowName)
{
	if (!CurveTable)
	{
		return false;
	}
	CurveTable->AddSimpleCurve(RowName);
	return true;
}

TArray<FName> UDataTableFunctionLibrary::GetCurveTableRowNames(UCurveTable* CurveTable)
{
	TArray<FName> RowNames;
	if (CurveTable)
	{
		CurveTable->GetRowMap().GetKeys(RowNames);
	}
	return RowNames;
}

bool UDataTableFunctionLibrary::SetCurveTableRowDefault(UCurveTable* CurveTable, FName RowName, float DefaultValue)
{
	if (!CurveTable)
	{
		return false;
	}
	FString Errors;
	FRealCurve* Curve = CurveTable->FindCurve(RowName, Errors, false);
	if (!Curve)
	{
		return false;
	}
	Curve->DefaultValue = DefaultValue;
	return true;
}

bool UDataTableFunctionLibrary::AddCurveTableKey(UCurveTable* CurveTable, FName RowName, float InTime, float InValue)
{
	if (!CurveTable)
	{
		return false;
	}
	FString Errors;
	FRealCurve* Curve = CurveTable->FindCurve(RowName, Errors, false);
	if (!Curve)
	{
		return false;
	}
	Curve->AddKey(InTime, InValue);
	return true;
}

TArray<FSimpleCurveKey> UDataTableFunctionLibrary::GetCurveTableKeys(UCurveTable* CurveTable, FName RowName)
{
	TArray<FSimpleCurveKey> Keys;
	if (CurveTable && CurveTable->GetCurveTableMode() == ECurveTableMode::SimpleCurves)
	{
		FString Context;
		FSimpleCurve* Curve = CurveTable->FindSimpleCurve(RowName, Context);
		if (Curve)
		{
			Keys = Curve->Keys;
		}
	}
	return Keys;
}

bool UDataTableFunctionLibrary::RemoveCurveTableRow(UCurveTable* CurveTable, FName RowName)
{
	if (!CurveTable)
	{
		return false;
	}
	CurveTable->RemoveRow(RowName);
	return true;

}

bool UDataTableFunctionLibrary::RenameCurveTableRow(
	UCurveTable* CurveTable, const FName& RowName, const FName& NewRowName)
{
	if (!CurveTable)
	{
		return false;
	}
	CurveTable->RenameRow(RowName, NewRowName);
	return true;
}

const UScriptStruct* UDataTableFunctionLibrary::GetDataTableRowStruct(const UDataTable* Table)
{
	return Table
		? Table->GetRowStruct()
		: nullptr;
}

bool UDataTableFunctionLibrary::DoesDataTableRowExist(const UDataTable* Table, FName RowName)
{
	if (!Table)
	{
		return false;
	}
	else if (Table->RowStruct == nullptr)
	{
		return false;
	}
	return Table->GetRowMap().Find(RowName) != nullptr;
}

TArray<FString> UDataTableFunctionLibrary::GetDataTableColumnAsString(const UDataTable* DataTable, FName PropertyName)
{
	if (DataTable && PropertyName != NAME_None)
	{
		EDataTableExportFlags ExportFlags = EDataTableExportFlags::None;
		return DataTableUtils::GetColumnDataAsString(DataTable, PropertyName, ExportFlags);
	}
	return TArray<FString>();
}

bool UDataTableFunctionLibrary::Generic_GetDataTableRowFromName(const UDataTable* Table, FName RowName, void* OutRowPtr)
{
	bool bFoundRow = false;

	if (OutRowPtr && Table)
	{
		void* RowPtr = Table->FindRowUnchecked(RowName);

		if (RowPtr != nullptr)
		{
			const UScriptStruct* StructType = Table->GetRowStruct();

			if (StructType != nullptr)
			{
				StructType->CopyScriptStruct(OutRowPtr, RowPtr);
				bFoundRow = true;
			}
		}
	}

	return bFoundRow;
}

bool UDataTableFunctionLibrary::GetDataTableRowFromName(UDataTable* Table, FName RowName, FTableRowBase& OutRow)
{
	// We should never hit this!  stubs to avoid NoExport on the class.
	check(0);
	return false;
}

DEFINE_FUNCTION(UDataTableFunctionLibrary::execGetDataTableRowFromName)
{
    P_GET_OBJECT(UDataTable, Table);
    P_GET_PROPERTY(FNameProperty, RowName);
        
    Stack.StepCompiledIn<FStructProperty>(NULL);
    void* OutRowPtr = Stack.MostRecentPropertyAddress;

	P_FINISH;
	bool bSuccess = false;
		
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	if (!Table)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			NSLOCTEXT("GetDataTableRow", "MissingTableInput", "Failed to resolve the table input. Be sure the DataTable is valid.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if(StructProp && OutRowPtr)
	{
		UScriptStruct* OutputType = StructProp->Struct;
		const UScriptStruct* TableType  = Table->GetRowStruct();
		
		if (OutputType == TableType)
		{
			P_NATIVE_BEGIN;
			bSuccess = Generic_GetDataTableRowFromName(Table, RowName, OutRowPtr);
			P_NATIVE_END;
		}
		else
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::AccessViolation,
				FText::Format(NSLOCTEXT("GetDataTableRow", "IncompatibleProperty", "Incompatible output parameter; the data table's type ({0}) is not the same as the return type ({1})."), FText::AsCultureInvariant(GetPathNameSafe(TableType)), FText::AsCultureInvariant(OutputType->GetPathName()))
				);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
	}
	else
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			NSLOCTEXT("GetDataTableRow", "MissingOutputProperty", "Failed to resolve the output parameter for GetDataTableRow.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	*(bool*)RESULT_PARAM = bSuccess;
}

void UDataTableFunctionLibrary::GetDataTableRowNames(const UDataTable* Table, TArray<FName>& OutRowNames)
{
	if (Table)
	{
		OutRowNames = Table->GetRowNames();
	}
	else
	{
		OutRowNames.Empty();
	}
}

void UDataTableFunctionLibrary::GetDataTableColumnNames(const UDataTable* Table, TArray<FName>& OutColumnNames)
{
	if (Table && Table->GetRowStruct())
	{
		OutColumnNames = DataTableUtils::GetStructPropertyNames(Table->GetRowStruct());
	}
	else
	{
		OutColumnNames.Empty();
	}
}

void UDataTableFunctionLibrary::GetDataTableColumnExportNames(const UDataTable* Table, TArray<FString>& OutExportColumnNames)
{
	OutExportColumnNames.Empty();

	if (Table && Table->GetRowStruct())
	{
		for (TFieldIterator<const FProperty> It(Table->GetRowStruct()); It; ++It)
		{
			OutExportColumnNames.Add(DataTableUtils::GetPropertyExportName(*It));
		}
	}
}

bool UDataTableFunctionLibrary::GetDataTableColumnNameFromExportName(const UDataTable* Table, const FString& ColumnExportName, FName& OutColumnName)
{
	if (Table && Table->GetRowStruct())
	{
		for (TFieldIterator<const FProperty> It(Table->GetRowStruct()); It; ++It)
		{
			const FString PropertyExportName = DataTableUtils::GetPropertyExportName(*It);
			if (PropertyExportName == ColumnExportName)
			{
				OutColumnName = It->GetFName();
				return true;
			}
		}
	}

	return false;
}


#if WITH_EDITOR
bool UDataTableFunctionLibrary::FillDataTableFromCSVString(UDataTable* DataTable, const FString& InString, UScriptStruct* ImportRowStruct)
{
	if (!DataTable)
	{
		UE_LOGF(LogDataTable, Error, "FillDataTableFromCSVString - The DataTable is invalid.");
		return false;
	}

	UCSVImportFactory* ImportFactory = NewObject<UCSVImportFactory>();
	ImportFactory->AutomatedImportSettings.bForceAutomatedImport = ImportRowStruct != nullptr;
	ImportFactory->AutomatedImportSettings.ImportRowStruct = ImportRowStruct;

	bool bWasCancelled = false;
	const TCHAR* Buffer = *InString;
	UObject* Result = ImportFactory->FactoryCreateText(DataTable->GetClass()
		, DataTable->GetOuter()
		, DataTable->GetFName()
		, DataTable->GetFlags()
		, nullptr
		, TEXT("csv")
		, Buffer
		, Buffer + InString.Len()
		, nullptr
		, bWasCancelled);

	return Result != nullptr && !bWasCancelled;
}

bool UDataTableFunctionLibrary::FillDataTableFromCSVFile(UDataTable* DataTable, const FString& InFilePath, UScriptStruct* ImportRowStruct)
{
	if (!DataTable)
	{
		UE_LOGF(LogDataTable, Error, "FillDataTableFromCSVFile - The DataTable is invalid.");
		return false;
	}

	if (!IFileManager::Get().FileExists(*InFilePath))
	{
		UE_LOGF(LogDataTable, Error, "FillDataTableFromCSVFile - The file '%ls' doesn't exist.", *InFilePath);
		return false;
	}

	DataTable->AssetImportData->Update(InFilePath);

	UCSVImportFactory* ImportFactory = NewObject<UCSVImportFactory>();
	ImportFactory->AutomatedImportSettings.bForceAutomatedImport = ImportRowStruct != nullptr;
	ImportFactory->AutomatedImportSettings.ImportRowStruct = ImportRowStruct;
	
	return ImportFactory->ReimportCSV(DataTable) == EReimportResult::Succeeded;
}

bool UDataTableFunctionLibrary::FillDataTableFromJSONString(UDataTable* DataTable, const FString& InString, UScriptStruct* ImportRowStruct)
{
	if (!DataTable)
	{
		UE_LOGF(LogDataTable, Error, "FillDataTableFromJSONString - The DataTable is invalid.");
		return false;
	}

	UCSVImportFactory* ImportFactory = NewObject<UCSVImportFactory>();
	ImportFactory->AutomatedImportSettings.bForceAutomatedImport = ImportRowStruct != nullptr;
	ImportFactory->AutomatedImportSettings.ImportRowStruct = ImportRowStruct;

	bool bWasCancelled = false;
	const TCHAR* Buffer = *InString;
	UObject* Result = ImportFactory->FactoryCreateText(DataTable->GetClass()
		, DataTable->GetOuter()
		, DataTable->GetFName()
		, DataTable->GetFlags()
		, nullptr
		, TEXT("json")
		, Buffer
		, Buffer + InString.Len()
		, nullptr
		, bWasCancelled);

	return Result != nullptr && !bWasCancelled;
}

bool UDataTableFunctionLibrary::FillDataTableFromJSONFile(UDataTable* DataTable, const FString& InFilePath, UScriptStruct* ImportRowStruct)
{
	if (!DataTable)
	{
		UE_LOGF(LogDataTable, Error, "FillDataTableFromJSONFile - The DataTable is invalid.");
		return false;
	}

	if (!IFileManager::Get().FileExists(*InFilePath))
	{
		UE_LOGF(LogDataTable, Error, "FillDataTableFromJSONFile - The file '%ls' doesn't exist.", *InFilePath);
		return false;
	}

	if (!InFilePath.EndsWith(TEXT(".json")))
	{
		UE_LOGF(LogDataTable, Error, "FillDataTableFromJSONFile - The file is not a JSON file.");
		return false;
	}

	DataTable->AssetImportData->Update(InFilePath);

	UCSVImportFactory* ImportFactory = NewObject<UCSVImportFactory>();
	ImportFactory->AutomatedImportSettings.bForceAutomatedImport = ImportRowStruct != nullptr;
	ImportFactory->AutomatedImportSettings.ImportRowStruct = ImportRowStruct;
	
	return ImportFactory->ReimportCSV(DataTable) == EReimportResult::Succeeded;
}

bool UDataTableFunctionLibrary::ExportDataTableToCSVString(const UDataTable* DataTable, FString& OutCSVString)
{
	if (!DataTable || !DataTable->GetRowStruct())
	{
		UE_LOGF(LogDataTable, Error, "ExportDataTableToCSVString - The DataTable is invalid.");
		return false;
	}

	OutCSVString = DataTable->GetTableAsCSV();
	return true;
}

bool UDataTableFunctionLibrary::ExportDataTableToCSVFile(const UDataTable* DataTable, const FString& CSVFilePath)
{
	if (!DataTable || !DataTable->GetRowStruct())
	{
		UE_LOGF(LogDataTable, Error, "ExportDataTableToCSVFile - The DataTable is invalid.");
		return false;
	}

	if (!FFileHelper::SaveStringToFile(DataTable->GetTableAsCSV(), *CSVFilePath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogDataTable, Error, "ExportDataTableToCSVFile - Failed to write file '%ls'.", *CSVFilePath);
		return false;
	}

	return true;
}

bool UDataTableFunctionLibrary::ExportDataTableToJSONString(const UDataTable* DataTable, FString& OutJSONString)
{
	if (!DataTable || !DataTable->GetRowStruct())
	{
		UE_LOGF(LogDataTable, Error, "ExportDataTableToJSONString - The DataTable is invalid.");
		return false;
	}

	OutJSONString = DataTable->GetTableAsJSON(EDataTableExportFlags::UseJsonObjectsForStructs);
	return true;
}

bool UDataTableFunctionLibrary::ExportDataTableToJSONFile(const UDataTable* DataTable, const FString& JSONFilePath)
{
	if (!DataTable || !DataTable->GetRowStruct())
	{
		UE_LOGF(LogDataTable, Error, "ExportDataTableToJSONFile - The DataTable is invalid.");
		return false;
	}

	if (!FFileHelper::SaveStringToFile(DataTable->GetTableAsJSON(EDataTableExportFlags::UseJsonObjectsForStructs), *JSONFilePath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogDataTable, Error, "ExportDataTableToJSONFile - Failed to write file '%ls'.", *JSONFilePath);
		return false;
	}

	return true;
}

void UDataTableFunctionLibrary::AddDataTableRow(UDataTable* const DataTable, const FName& RowName, const FTableRowBase& RowData)
{
	if (!DataTable)
	{
		UE_LOGF(LogDataTable, Error, "AddDataTableRow - The DataTable is invalid.");
		return;
	}

	DataTable->Modify();
	DataTable->AddRow(RowName, RowData);
}

DEFINE_FUNCTION(UDataTableFunctionLibrary::execAddDataTableRow)
{
	P_GET_OBJECT(UDataTable, DataTable);
	P_GET_PROPERTY(FNameProperty, RowName);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	const uint8* const RowData = Stack.MostRecentPropertyAddress;
	const FStructProperty* const StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	if (!DataTable)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			NSLOCTEXT("AddDataTableRow", "MissingTableInput", "Failed to resolve the DataTable parameter.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if (StructProp && RowData)
	{
		const UScriptStruct* const RowType = StructProp->Struct;
		const UScriptStruct* const TableType = DataTable->GetRowStruct();

		// If the row type is compatible with the table type ...
		if (RowType == TableType)
		{
			P_NATIVE_BEGIN;
			DataTable->Modify();
			DataTable->AddRow(RowName, RowData, RowType);
			P_NATIVE_END;
		}
		else
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::AccessViolation,
				FText::Format(NSLOCTEXT("AddDataTableRow", "IncompatibleProperty", "Incompatible RowData parameter; the data table's type ({0}) is not the same as the RowData type ({1})."), FText::AsCultureInvariant(GetPathNameSafe(TableType)), FText::AsCultureInvariant(RowType->GetPathName()))
			);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
	}
	else
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			NSLOCTEXT("AddDataTableRow", "MissingOutputProperty", "Failed to resolve the RowData parameter.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
}

void UDataTableFunctionLibrary::RemoveDataTableRow(UDataTable* DataTable, const FName& RowName)
{
	if (!DataTable)
	{
		UE_LOGF(LogDataTable, Error, "RemoveDataTableRow - The DataTable is invalid.");
		return;
	}

	if (DataTable->FindRowUnchecked(RowName) != nullptr)
	{
		DataTable->Modify();
		DataTable->RemoveRow(RowName);
	}
	else
	{
		UE_LOGF(LogDataTable, Log, "RemoveDataTableRow - Row %ls not found is %ls", *RowName.ToString(), *DataTable->GetName());
	}
}
#endif //WITH_EDITOR
