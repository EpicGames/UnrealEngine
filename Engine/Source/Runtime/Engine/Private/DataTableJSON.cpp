// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableJSON.h"
#include "Engine/DataTable.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/JsonSerializer.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/CoreRedirects.h"
#include "UObject/EnumProperty.h"


namespace UE::DataTableJSON
{
	constexpr const TCHAR* InstancedStructTypeFieldName = TEXT("_structType");

	const TCHAR* JSONTypeToString(const EJson InType)
	{
		switch(InType)
		{
		case EJson::None:
			return TEXT("None");
		case EJson::Null:
			return TEXT("Null");
		case EJson::String:
			return TEXT("String");
		case EJson::Number:
			return TEXT("Number");
		case EJson::Boolean:
			return TEXT("Boolean");
		case EJson::Array:
			return TEXT("Array");
		case EJson::Object:
			return TEXT("Object");
		default:
			return TEXT("Unknown");
		}
	}

#if WITH_EDITOR

	template <typename CharType>
	void WriteJSONObjectStartWithOptionalIdentifier(typename TDataTableExporterJSON<CharType>::FDataTableJsonWriter& InJsonWriter, const FString* InIdentifier)
	{
		if (InIdentifier)
		{
			InJsonWriter.WriteObjectStart(*InIdentifier);
		}
		else
		{
			InJsonWriter.WriteObjectStart();
		}
	}

	template <typename CharType, typename ValueType>
	void WriteJSONValueWithOptionalIdentifier(typename TDataTableExporterJSON<CharType>::FDataTableJsonWriter& InJsonWriter, const FString* InIdentifier, const ValueType InValue)
	{
		if (InIdentifier)
		{
			InJsonWriter.WriteValue(*InIdentifier, InValue);
		}
		else
		{
			InJsonWriter.WriteValue(InValue);
		}
	}

	/**
	 * Console variable to be able to run the previous implementation of the exporter
	 * in case issues get reported and validate potential regression.
	 * We could eventually get rid of the FieldIterator implementation.
	 */
	bool bExportUsingPropertyVisitor = true;
	static FAutoConsoleVariableRef CVarRuntimeValidationContext(
		TEXT("DataTableJSON.ExportUsingPropertyVisitor"),
		bExportUsingPropertyVisitor,
		TEXT("True indicates that the PropertyVisitor will be used to export the data table row including support for new types like instanced struct."
				" Otherwise the legacy implementation based on the FieldIterator is used.")
	);

#endif	// WITH_EDITOR

}

FString DataTableJSONUtils::GetKeyFieldName(const UDataTable& InDataTable)
{
	FString ExplicitString = InDataTable.ImportKeyField;
	if (ExplicitString.IsEmpty())
	{
		return TEXT("Name");
	}
	else
	{
		return ExplicitString;
	}
}


#if WITH_EDITOR

FDataTableExporterJSON::FDataTableExporterJSON(const EDataTableExportFlags InDTExportFlags, FString& OutExportText)
	: TDataTableExporterJSON<TCHAR>(InDTExportFlags, TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&OutExportText))
{
	bJsonWriterNeedsClose = true;
}

template<typename CharType>
TDataTableExporterJSON<CharType>::TDataTableExporterJSON(const EDataTableExportFlags InDTExportFlags, TSharedRef<FDataTableJsonWriter> InJsonWriter)
	: DTExportFlags(InDTExportFlags)
	, JsonWriter(InJsonWriter)
	, bJsonWriterNeedsClose(false)
{
}

template<typename CharType>
TDataTableExporterJSON<CharType>::~TDataTableExporterJSON()
{
	if (bJsonWriterNeedsClose)
	{
		JsonWriter->Close();
	}
}

template<typename CharType>
bool TDataTableExporterJSON<CharType>::WriteTable(const UDataTable& InDataTable)
{
	if (!InDataTable.RowStruct)
	{
		return false;
	}

	FString KeyField = DataTableJSONUtils::GetKeyFieldName(InDataTable);
	JsonWriter->WriteArrayStart();

	// Iterate over rows
	for (auto RowIt = InDataTable.GetRowMap().CreateConstIterator(); RowIt; ++RowIt)
	{
		JsonWriter->WriteObjectStart();
		{
			// RowName
			const FName RowName = RowIt.Key();
			JsonWriter->WriteValue(KeyField, RowName.ToString());

			// Now the values
			uint8* RowData = RowIt.Value();
			WriteRow(InDataTable.RowStruct, RowData, &KeyField);
		}
		JsonWriter->WriteObjectEnd();
	}

	JsonWriter->WriteArrayEnd();

	return true;
}

template<typename CharType>
bool TDataTableExporterJSON<CharType>::WriteTableAsObject(const UDataTable& InDataTable)
{
	if (!InDataTable.RowStruct)
	{
		return false;
	}

	JsonWriter->WriteObjectStart(InDataTable.GetName());

	// Iterate over rows
	for (auto RowIt = InDataTable.GetRowMap().CreateConstIterator(); RowIt; ++RowIt)
	{
		// RowName
		const FName RowName = RowIt.Key();
		JsonWriter->WriteObjectStart(RowName.ToString());
		{
			// Now the values
			uint8* RowData = RowIt.Value();
			WriteRow(InDataTable.RowStruct, RowData);
		}
		JsonWriter->WriteObjectEnd();
	}
	JsonWriter->WriteObjectEnd();

	return true;
}

template<typename CharType>
bool TDataTableExporterJSON<CharType>::WriteRow(const UScriptStruct* InRowStruct, const void* InRowData, const FString* FieldToSkip)
{
	if (!InRowStruct)
	{
		return false;
	}

	return WriteStruct(InRowStruct, InRowData, FieldToSkip);
}

template<typename CharType>
bool TDataTableExporterJSON<CharType>::WriteStruct(const UScriptStruct* InStruct, const void* InStructData, const FString* FieldToSkip)
{
	if (UE::DataTableJSON::bExportUsingPropertyVisitor)
	{
		return InStruct ? WriteStructUsingVisitor(InStruct, InStructData, FieldToSkip) : false;
	}

	return WriteStructUsingFieldIterator(InStruct, InStructData, FieldToSkip);
}

template <typename CharType>
bool TDataTableExporterJSON<CharType>::WriteStructUsingFieldIterator(
	const UScriptStruct* InStruct
	, const void* InStructData
	, const FString* FieldToSkip)
{
	// FInstancedStruct: emit a runtime type marker and recurse into the inner struct. 
	if (InStruct == TBaseStructure<FInstancedStruct>::Get())
	{
		const FInstancedStruct* Instanced = static_cast<const FInstancedStruct*>(InStructData);
		if (const UScriptStruct* RuntimeStruct = Instanced->GetScriptStruct())
		{
			if (const uint8* RuntimeMemory = Instanced->GetMemory())
			{
				JsonWriter->WriteValue(UE::DataTableJSON::InstancedStructTypeFieldName, RuntimeStruct->GetPathName());
				return WriteStructUsingFieldIterator(RuntimeStruct, RuntimeMemory, FieldToSkip);
			}
		}

		return true;
	}

	for (TFieldIterator<const FProperty> It(InStruct); It; ++It)
	{
		const FProperty* BaseProp = *It;
		check(BaseProp);

		const FString Identifier = DataTableUtils::GetPropertyExportName(BaseProp, DTExportFlags);
		if (FieldToSkip && *FieldToSkip == Identifier)
		{
			// Skip this field
			continue;
		}
 
		if (BaseProp->ArrayDim == 1)
		{
			const void* Data = BaseProp->ContainerPtrToValuePtr<void>(InStructData, 0);
			WriteStructEntry(InStructData, BaseProp, Data);
		}
		else
		{
			JsonWriter->WriteArrayStart(Identifier);

			for (int32 ArrayEntryIndex = 0; ArrayEntryIndex < BaseProp->ArrayDim; ++ArrayEntryIndex)
			{
				const void* Data = BaseProp->ContainerPtrToValuePtr<void>(InStructData, ArrayEntryIndex);
				WriteContainerEntry(BaseProp, Data);
			}

			JsonWriter->WriteArrayEnd();
		}
	}

	return true;
}

template<typename CharType>
bool TDataTableExporterJSON<CharType>::WriteStructEntry(const void* InRowData, const FProperty* InProperty, const void* InPropertyData)
{
	const FString Identifier = DataTableUtils::GetPropertyExportName(InProperty, DTExportFlags);

	if (const FEnumProperty* EnumProp = CastField<const FEnumProperty>(InProperty))
	{
		const FString PropertyValue = DataTableUtils::GetPropertyValueAsString(EnumProp, (uint8*)InRowData, DTExportFlags);
		JsonWriter->WriteValue(Identifier, PropertyValue);
	}
	else if (const FNumericProperty *NumProp = CastField<const FNumericProperty>(InProperty))
	{
		if (NumProp->IsEnum())
		{
			const FString PropertyValue = DataTableUtils::GetPropertyValueAsString(InProperty, (uint8*)InRowData, DTExportFlags);
			JsonWriter->WriteValue(Identifier, PropertyValue);
		}
		else if (NumProp->IsInteger())
		{
			const int64 PropertyValue = NumProp->GetSignedIntPropertyValue(InPropertyData);
			JsonWriter->WriteValue(Identifier, PropertyValue);
		}
		else if (NumProp->IsA(FFloatProperty::StaticClass()))
		{
			const float PropertyValue = (float)NumProp->GetFloatingPointPropertyValue(InPropertyData);
			JsonWriter->WriteValue(Identifier, PropertyValue);
		}
		else
		{
			const double PropertyValue = NumProp->GetFloatingPointPropertyValue(InPropertyData);
			JsonWriter->WriteValue(Identifier, PropertyValue);
		}
	}
	else if (const FBoolProperty* BoolProp = CastField<const FBoolProperty>(InProperty))
	{
		const bool PropertyValue = BoolProp->GetPropertyValue(InPropertyData);
		JsonWriter->WriteValue(Identifier, PropertyValue);
	}
	else if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(InProperty))
	{
		JsonWriter->WriteArrayStart(Identifier);

		FScriptArrayHelper ArrayHelper(ArrayProp, InPropertyData);
		for (int32 ArrayEntryIndex = 0; ArrayEntryIndex < ArrayHelper.Num(); ++ArrayEntryIndex)
		{
			const uint8* ArrayEntryData = ArrayHelper.GetRawPtr(ArrayEntryIndex);
			WriteContainerEntry(ArrayProp->Inner, ArrayEntryData);
		}

		JsonWriter->WriteArrayEnd();
	}
	else if (const FSetProperty* SetProp = CastField<const FSetProperty>(InProperty))
	{
		JsonWriter->WriteArrayStart(Identifier);

		FScriptSetHelper SetHelper(SetProp, InPropertyData);
		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			const uint8* SetEntryData = SetHelper.GetElementPtr(It);
			WriteContainerEntry(SetHelper.GetElementProperty(), SetEntryData);
		}

		JsonWriter->WriteArrayEnd();
	}
	else if (const FMapProperty* MapProp = CastField<const FMapProperty>(InProperty))
	{
		JsonWriter->WriteObjectStart(Identifier);

		FScriptMapHelper MapHelper(MapProp, InPropertyData);
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			const uint8* MapKeyData = MapHelper.GetKeyPtr(It);
			const uint8* MapValueData = MapHelper.GetValuePtr(It);

			// JSON object keys must always be strings
			const FString KeyValue = DataTableUtils::GetPropertyValueAsStringDirect(MapHelper.GetKeyProperty(), (uint8*)MapKeyData, DTExportFlags);
			WriteContainerEntry(MapHelper.GetValueProperty(), MapValueData, &KeyValue);
		}

		JsonWriter->WriteObjectEnd();
	}
	else if (const FStructProperty* StructProp = CastField<const FStructProperty>(InProperty))
	{
		if (EnumHasAnyFlags(DTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
		{
			JsonWriter->WriteObjectStart(Identifier);
			WriteStruct(StructProp->Struct, InPropertyData);
			JsonWriter->WriteObjectEnd();
		}
		else
		{
			const FString PropertyValue = DataTableUtils::GetPropertyValueAsString(InProperty, (uint8*)InRowData, DTExportFlags);
			JsonWriter->WriteValue(Identifier, PropertyValue);
		}
	}
	else
	{
		const FString PropertyValue = DataTableUtils::GetPropertyValueAsString(InProperty, (uint8*)InRowData, DTExportFlags);
		JsonWriter->WriteValue(Identifier, PropertyValue);
	}

	return true;
}

template<typename CharType>
bool TDataTableExporterJSON<CharType>::WriteContainerEntry(const FProperty* InProperty, const void* InPropertyData, const FString* InIdentifier)
{
	using namespace UE::DataTableJSON;

	if (const FEnumProperty* EnumProp = CastField<const FEnumProperty>(InProperty))
	{
		const FString PropertyValue = DataTableUtils::GetPropertyValueAsStringDirect(InProperty, (uint8*)InPropertyData, DTExportFlags);
		WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
	}
	else if (const FNumericProperty *NumProp = CastField<const FNumericProperty>(InProperty))
	{
		if (NumProp->IsEnum())
		{
			const FString PropertyValue = DataTableUtils::GetPropertyValueAsStringDirect(InProperty, (uint8*)InPropertyData, DTExportFlags);
			WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
		}
		else if (NumProp->IsInteger())
		{
			const int64 PropertyValue = NumProp->GetSignedIntPropertyValue(InPropertyData);
			WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
		}
		else
		{
			const double PropertyValue = NumProp->GetFloatingPointPropertyValue(InPropertyData);
			WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
		}
	}
	else if (const FBoolProperty* BoolProp = CastField<const FBoolProperty>(InProperty))
	{
		const bool PropertyValue = BoolProp->GetPropertyValue(InPropertyData);
		WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
	}
	else if (const FStructProperty* StructProp = CastField<const FStructProperty>(InProperty))
	{
		if (EnumHasAnyFlags(DTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
		{
			WriteJSONObjectStartWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier);
			WriteStruct(StructProp->Struct, InPropertyData);
			JsonWriter->WriteObjectEnd();
		}
		else
		{
			const FString PropertyValue = DataTableUtils::GetPropertyValueAsStringDirect(InProperty, (uint8*)InPropertyData, DTExportFlags);
			WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
		}
	}
	else if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(InProperty))
	{
		// Cannot nest arrays
		return false;
	}
	else if (const FSetProperty* SetProp = CastField<const FSetProperty>(InProperty))
	{
		// Cannot nest sets
		return false;
	}
	else if (const FMapProperty* MapProp = CastField<const FMapProperty>(InProperty))
	{
		// Cannot nest maps
		return false;
	}
	else
	{
		const FString PropertyValue = DataTableUtils::GetPropertyValueAsStringDirect(InProperty, (uint8*)InPropertyData, DTExportFlags);
		WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
	}

	return true;
}

template <typename CharType>
bool TDataTableExporterJSON<CharType>::WriteStructUsingVisitor(TNotNull<const UScriptStruct*> InStruct, const void* InStructData
	, const FString* FieldToSkip)
{
	using namespace UE::DataTableJSON;

	// Keep a stack of identifier overrides since when entering arrays we don't want struct to output their identifier
	// We always push override for structs (when UseJsonObjectsForStructs is set in the export flags) and arrays 
	// so an empty override list or an unset value on top of the stack
	// indicates that no override is effective and the property export name should be used.
	TArray<TOptional<FString>> IdentifierOverrides;

	auto PushFunc = [this, &IdentifierOverrides](const FPropertyVisitorInfo& InVisitorInfo)
		{
			const FProperty* Property = InVisitorInfo.Property;
			const FString Identifier = DataTableUtils::GetPropertyExportName(Property, DTExportFlags);

			if (InVisitorInfo.PropertyInfo == EPropertyVisitorInfoType::MapKey
				|| InVisitorInfo.PropertyInfo == EPropertyVisitorInfoType::MapValue)
			{
				// Special case for map keys since we won't write them but use their values as string when writing the map values.
				// In order to do so we don't push a new identifier, and we will set it when visiting the key value.
				// Then we can reset when popping the values.
				return;
			}

			// Process static array first to properly handle arrays of structs
			if (Property->ArrayDim != 1)
			{
				ensure(InVisitorInfo.PropertyInfo == EPropertyVisitorInfoType::StaticArrayIndex);
				if (InVisitorInfo.Index == 0)
				{
					JsonWriter->WriteArrayStart(Identifier);

					// Push override (empty identifier) for array elements
					IdentifierOverrides.AddDefaulted_GetRef().Emplace(FString{});
				}
			}

			if (Property->IsA<FMapProperty>())
			{
				JsonWriter->WriteObjectStart(Identifier);
			}
			else if (Property->IsA<FArrayProperty>()
				|| Property->IsA<FSetProperty>())
			{
				JsonWriter->WriteArrayStart(Identifier);

				// Push override (empty identifier) for array elements
				IdentifierOverrides.AddDefaulted_GetRef().Emplace(FString{});
			}
			else if (Property->IsA<FStructProperty>())
			{
				if (EnumHasAnyFlags(DTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
				{
					if (IdentifierOverrides.Num() && IdentifierOverrides.Top().IsSet())
					{
						const FString* Override = IdentifierOverrides.Top().GetValue().IsEmpty() ? nullptr : &IdentifierOverrides.Top().GetValue();
						WriteJSONObjectStartWithOptionalIdentifier<CharType>(*JsonWriter, Override);
					}
					else
					{
						JsonWriter->WriteObjectStart(Identifier);
					}
					// Push a "no override" entry when entering a struct to export it as json object. If UseJsonObjectsForStructs is not used the struct is saved as string value.
					IdentifierOverrides.AddDefaulted();
				}
			}
		};

	auto PopFunc = [this, &IdentifierOverrides](const FPropertyVisitorInfo& InVisitorInfo)
		{
			// Map keys don't do anything when pushed in the visitor scope, so nothing to do here
			if (InVisitorInfo.PropertyInfo == EPropertyVisitorInfoType::MapKey)
			{
				return;
			}

			// Map element is done after writing its value; we can reset the override set by the key and close its WriteObjectStart.
			if (InVisitorInfo.PropertyInfo == EPropertyVisitorInfoType::MapValue)
			{
				JsonWriter->WriteObjectEnd();
				return;
			}

			const FProperty* Property = InVisitorInfo.Property;

			if (Property->IsA<FMapProperty>())
			{
				JsonWriter->WriteObjectEnd();
			}
			else if (Property->IsA<FArrayProperty>()
				|| Property->IsA<FSetProperty>())
			{
				JsonWriter->WriteArrayEnd();
				IdentifierOverrides.Pop();
			}
			else if (Property->IsA<FStructProperty>())
			{
				if (EnumHasAnyFlags(DTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
				{
					// Only close object and pop override if saving structs as json objects. If not they are saved as string values so no need to close it
					JsonWriter->WriteObjectEnd();
					IdentifierOverrides.Pop();
				}
			}

			// Process static array last to properly close an array of structs
			if (Property->ArrayDim != 1)
			{
				ensure(InVisitorInfo.PropertyInfo == EPropertyVisitorInfoType::StaticArrayIndex);
				if (InVisitorInfo.Index + 1 == Property->ArrayDim)
				{
					JsonWriter->WriteArrayEnd();
					IdentifierOverrides.Pop();
				}
			}
		};

	// Visitor
	FPropertyVisitorPath Path(PushFunc, PopFunc);
	FPropertyVisitorData Data(const_cast<void*>(InStructData), /*ParentStructData*/nullptr);
	FPropertyVisitorContext Context(Path, Data, FPropertyVisitorContext::EScope::All);

	InStruct->Visit(Context,
		[FieldToSkip, this, &IdentifierOverrides](const FPropertyVisitorContext& InContext)-> EPropertyVisitorControlFlow
		{
			const FProperty* Property = InContext.Path.Top().Property;
			if (FieldToSkip && *FieldToSkip == DataTableUtils::GetPropertyExportName(Property, DTExportFlags))
			{
				// Skip this field
				return EPropertyVisitorControlFlow::StepOver;
			}

			// Special case for map when we use the key to set the identifier for the values
			if (InContext.Path.Top().PropertyInfo == EPropertyVisitorInfoType::MapKey)
			{
				const FString Identifier = (DataTableUtils::GetPropertyValueAsString(Property, static_cast<uint8*>(InContext.Data.PropertyData), DTExportFlags));
				JsonWriter->WriteObjectStart(Identifier);
				return EPropertyVisitorControlFlow::StepOver;
			}

			// FInstancedStruct: emit a "_structType" marker at the current level and recurse into the struct
			if (const FStructProperty* StructProp = CastField<const FStructProperty>(Property))
			{
				if (StructProp->Struct == TBaseStructure<FInstancedStruct>::Get()
					&& EnumHasAnyFlags(DTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
				{
					const FInstancedStruct* Instanced = static_cast<const FInstancedStruct*>(InContext.Data.PropertyData);
					if (const UScriptStruct* RuntimeStruct = Instanced->GetScriptStruct())
					{
						JsonWriter->WriteValue(InstancedStructTypeFieldName, RuntimeStruct->GetPathName());
						if (Instanced->GetMemory())
						{
							WriteStructUsingVisitor(RuntimeStruct, Instanced->GetMemory(), /*FieldToSkip=*/nullptr);
						}
					}
					return EPropertyVisitorControlFlow::StepOver;
				}
			}

			// Use override identifier if one is set on the stack
			if (IdentifierOverrides.Num() && IdentifierOverrides.Top().IsSet())
			{
				const FString* Override = IdentifierOverrides.Top().GetValue().IsEmpty() ? nullptr : &IdentifierOverrides.Top().GetValue();
				return WriteVisitedEntry(Property, InContext.Data.PropertyData, Override);
			}

			const FString ExportName = DataTableUtils::GetPropertyExportName(Property, DTExportFlags);
			return WriteVisitedEntry(Property, InContext.Data.PropertyData, &ExportName);
		});

	return true;
}

template<typename CharType>
EPropertyVisitorControlFlow TDataTableExporterJSON<CharType>::WriteVisitedEntry(TNotNull<const FProperty*> InProperty, const void* InValueAddress, const FString* InIdentifier)
{
	using namespace UE::DataTableJSON;

	if (const FEnumProperty* EnumProp = CastField<const FEnumProperty>(InProperty))
	{
		const FString PropertyValue = DataTableUtils::GetPropertyValueAsStringDirect(EnumProp, (uint8*)InValueAddress, DTExportFlags);
		WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
	}
	else if (const FNumericProperty* NumProp = CastField<const FNumericProperty>(InProperty))
	{
		if (NumProp->IsEnum())
		{
			const FString PropertyValue = DataTableUtils::GetPropertyValueAsStringDirect(InProperty, (uint8*)InValueAddress, DTExportFlags);
			WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
		}
		else if (NumProp->IsInteger())
		{
			const int64 PropertyValue = NumProp->GetSignedIntPropertyValue(InValueAddress);
			WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
		}
		else if (NumProp->IsA<FFloatProperty>())
		{
			const float PropertyValue = (float)NumProp->GetFloatingPointPropertyValue(InValueAddress);
			WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
		}
		else
		{
			const double PropertyValue = NumProp->GetFloatingPointPropertyValue(InValueAddress);
			WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
		}
	}
	else if (const FBoolProperty* BoolProp = CastField<const FBoolProperty>(InProperty))
	{
		const bool PropertyValue = BoolProp->GetPropertyValue(InValueAddress);
		WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
	}
	else if (InProperty->IsA<const FStructProperty>())
	{
		if (EnumHasAnyFlags(DTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
		{
			return EPropertyVisitorControlFlow::StepInto;
		}

		const FString PropertyValue = DataTableUtils::GetPropertyValueAsStringDirect(InProperty, (uint8*)InValueAddress, DTExportFlags);
		WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
	}
	else if (InProperty->IsA<FArrayProperty>()
		|| InProperty->IsA<FSetProperty>()
		|| InProperty->IsA<FMapProperty>())
	{
		return EPropertyVisitorControlFlow::StepInto;
	}
	else
	{
		const FString PropertyValue = DataTableUtils::GetPropertyValueAsStringDirect(InProperty, (uint8*)InValueAddress, DTExportFlags);
		WriteJSONValueWithOptionalIdentifier<CharType>(*JsonWriter, InIdentifier, PropertyValue);
	}

	return EPropertyVisitorControlFlow::StepOver;
};

template class TDataTableExporterJSON<TCHAR>;
template class TDataTableExporterJSON<ANSICHAR>;

#endif // WITH_EDITOR


FDataTableImporterJSON::FDataTableImporterJSON(UDataTable& InDataTable, const FString& InJSONData, TArray<FString>& OutProblems)
	: DataTable(&InDataTable)
	, JSONData(InJSONData)
	, ImportProblems(OutProblems)
{
}

FDataTableImporterJSON::~FDataTableImporterJSON()
{
}

bool FDataTableImporterJSON::ReadTable()
{
	if (JSONData.IsEmpty())
	{
		ImportProblems.Add(TEXT("Input data is empty."));
		return false;
	}

	// Check we have a RowStruct specified
	if (!DataTable->RowStruct)
	{
		ImportProblems.Add(TEXT("No RowStruct specified."));
		return false;
	}

	TArray< TSharedPtr<FJsonValue> > ParsedTableRows;
	{
		const TSharedRef< TJsonReader<TCHAR> > JsonReader = TJsonReaderFactory<TCHAR>::Create(JSONData);
		if (!FJsonSerializer::Deserialize(JsonReader, ParsedTableRows) || ParsedTableRows.Num() == 0)
		{
			ImportProblems.Add(FString::Printf(TEXT("Failed to parse the JSON data. Error: %s"), *JsonReader->GetErrorMessage()));
			return false;
		}
	}

	// Empty existing data
	DataTable->EmptyTable();

	// Iterate over rows
	for (int32 RowIdx = 0; RowIdx < ParsedTableRows.Num(); ++RowIdx)
	{
		const TSharedPtr<FJsonValue>& ParsedTableRowValue = ParsedTableRows[RowIdx];
		TSharedPtr<FJsonObject> ParsedTableRowObject = ParsedTableRowValue->AsObject();
		if (!ParsedTableRowObject.IsValid())
		{
			ImportProblems.Add(FString::Printf(TEXT("Row '%d' is not a valid JSON object."), RowIdx));
			continue;
		}

		ReadRow(ParsedTableRowObject.ToSharedRef(), RowIdx);
	}

	DataTable->Modify(true);

	return true;
}

bool FDataTableImporterJSON::AddRow(bool bRemoveDuplicate, FName* OutRowName)
{
	if (JSONData.IsEmpty())
	{
		ImportProblems.Add(TEXT("Input data is empty."));
		return false;
	}

	// Check we have a RowStruct specified
	if (!DataTable->RowStruct)
	{
		ImportProblems.Add(TEXT("No RowStruct specified."));
		return false;
	}

	TSharedPtr<FJsonValue> ParsedTableRow;
	{
		const TSharedRef< TJsonReader<TCHAR> > JsonReader = TJsonReaderFactory<TCHAR>::Create(JSONData);
		if (!FJsonSerializer::Deserialize(JsonReader, ParsedTableRow) || !ParsedTableRow.IsValid())
		{
			ImportProblems.Add(FString::Printf(TEXT("Failed to parse the JSON data. Error: %s"), *JsonReader->GetErrorMessage()));
			return false;
		}
	}

	TSharedPtr<FJsonObject> ParsedTableRowObject = ParsedTableRow->AsObject();
	if (!ParsedTableRowObject.IsValid())
	{
		ImportProblems.Add(TEXT("Row is not a valid JSON object."));
		return false;
	}

	return ReadRow(ParsedTableRowObject.ToSharedRef(), 0, bRemoveDuplicate, OutRowName);
}

bool FDataTableImporterJSON::ReadRow(const TSharedRef<FJsonObject>& InParsedTableRowObject, const int32 InRowIdx, bool bRemoveDuplicate, FName* OutRowName)
{
	// Get row name
	FString RowKey = DataTableJSONUtils::GetKeyFieldName(*DataTable);
	FName RowName = DataTableUtils::MakeValidName(InParsedTableRowObject->GetStringField(RowKey));

	// Check its not 'none'
	if (RowName.IsNone())
	{
		ImportProblems.Add(FString::Printf(TEXT("Row '%d' missing key field '%s'."), InRowIdx, *RowKey));
		return false;
	}
	else if (OutRowName != nullptr)
	{
		*OutRowName = RowName;
	}

	// Check its not a duplicate
	if (DataTable->GetRowMap().Find(RowName) != nullptr)
	{
		if (bRemoveDuplicate)
		{
			DataTable->RemoveRowInternal(RowName);
		}
		else if (!DataTable->AllowDuplicateRowsOnImport())
		{
			ImportProblems.Add(FString::Printf(TEXT("Duplicate row name '%s'."), *RowName.ToString()));
			return false;
		}
	}

	// Detect any extra fields within the data for this row
	if (!DataTable->bIgnoreExtraFields)
	{
		TArray<FString> TempPropertyImportNames;
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& ParsedPropertyKeyValuePair : InParsedTableRowObject->Values)
		{
			if (FStringView(ParsedPropertyKeyValuePair.Key) == RowKey)
			{
				// Skip the row name, as that doesn't match a property
				continue;
			}

			FName PropName = DataTableUtils::MakeValidName(FString(ParsedPropertyKeyValuePair.Key));
			FProperty* ColumnProp = FindFProperty<FProperty>(DataTable->RowStruct, PropName);
			for (TFieldIterator<FProperty> It(DataTable->RowStruct); It && !ColumnProp; ++It)
			{
				DataTableUtils::GetPropertyImportNames(*It, TempPropertyImportNames);
				ColumnProp = TempPropertyImportNames.Contains(FString(ParsedPropertyKeyValuePair.Key)) ? *It : nullptr;
			}

			if (!ColumnProp)
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' cannot be found in struct '%s'."), *PropName.ToString(), *RowName.ToString(), *DataTable->RowStruct->GetName()));
			}
		}
	}

	// Allocate data to store information, using UScriptStruct to know its size
	uint8* RowData = (uint8*)FMemory::Malloc(DataTable->RowStruct->GetStructureSize());
	DataTable->RowStruct->InitializeStruct(RowData);
	// And be sure to call DestroyScriptStruct later

	// Add to row map
	DataTable->AddRowInternal(RowName, RowData);

	return ReadStruct(InParsedTableRowObject, DataTable->RowStruct, RowName, RowData);
}

bool FDataTableImporterJSON::ReadStruct(const TSharedRef<FJsonObject>& InParsedObject, UScriptStruct* InStruct, const FName InRowName, void* InStructData)
{
	// FInstancedStruct: resolve the struct type from the "_structType" marker
	// emitted by the exporter, then recurse into the inner struct.
	if (InStruct == TBaseStructure<FInstancedStruct>::Get())
	{
		FInstancedStruct* Instanced = static_cast<FInstancedStruct*>(InStructData);
		FString StructPathName;
		if (!InParsedObject->TryGetStringField(UE::DataTableJSON::InstancedStructTypeFieldName, StructPathName)
			|| StructPathName.IsEmpty())
		{
			// No marker - treat as empty InstancedStruct
			Instanced->Reset();
			return true;
		}

		// Honor core redirects
		FCoreRedirectObjectName OldName(StructPathName);
		FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(
			ECoreRedirectFlags::Type_Struct, OldName, ECoreRedirectMatchFlags::AllowPartialMatch);
		if (OldName != NewName)
		{
			StructPathName = NewName.ToString();
		}

		UScriptStruct* StructTypePtr = LoadObject<UScriptStruct>(nullptr, *StructPathName);
		if (!StructTypePtr)
		{
			ImportProblems.Add(FString::Printf(TEXT("Row '%s' InstancedStruct type '%s' could not be resolved."),
				*InRowName.ToString(), *StructPathName));
			Instanced->Reset();
			return false;
		}
		Instanced->InitializeAs(StructTypePtr);
		return ReadStruct(InParsedObject, StructTypePtr, InRowName, Instanced->GetMutableMemory());
	}

	// Now read in each property
	TArray<FString> TempPropertyImportNames;
	for (TFieldIterator<FProperty> It(InStruct); It; ++It)
	{
		FProperty* BaseProp = *It;
		check(BaseProp);

		const FString ColumnName = DataTableUtils::GetPropertyExportName(BaseProp);

		TSharedPtr<FJsonValue> ParsedPropertyValue;
		DataTableUtils::GetPropertyImportNames(BaseProp, TempPropertyImportNames);
		for (const FString& PropertyName : TempPropertyImportNames)
		{
			ParsedPropertyValue = InParsedObject->TryGetField(PropertyName);
			if (ParsedPropertyValue.IsValid())
			{
				break;
			}
		}

		if (!ParsedPropertyValue.IsValid())
		{
#if WITH_EDITOR
			// If the structure has specified the property as optional for import (gameplay code likely doing a custom fix-up or parse of that property),
			// then avoid warning about it
			static const FName DataTableImportOptionalMetadataKey("DataTableImportOptional");
			if (BaseProp->HasMetaData(DataTableImportOptionalMetadataKey))
			{
				continue;
			}
#endif // WITH_EDITOR

			if (!DataTable->bIgnoreMissingFields)
			{
				ImportProblems.Add(FString::Printf(TEXT("Row '%s' is missing an entry for '%s'."), *InRowName.ToString(), *ColumnName));
			}

			continue;
		}

		if (BaseProp->ArrayDim == 1)
		{
			void* Data = BaseProp->ContainerPtrToValuePtr<void>(InStructData, 0);
			ReadStructEntry(ParsedPropertyValue.ToSharedRef(), InRowName, ColumnName, InStructData, BaseProp, Data);
		}
		else
		{
			const TCHAR* const ParsedPropertyType = UE::DataTableJSON::JSONTypeToString(ParsedPropertyValue->Type);

			const TArray< TSharedPtr<FJsonValue> >* PropertyValuesPtr;
			if (!ParsedPropertyValue->TryGetArray(PropertyValuesPtr))
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected Array, got %s."), *ColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			if (BaseProp->ArrayDim != PropertyValuesPtr->Num())
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is a static sized array with %d elements, but we have %d values to import"), *ColumnName, *InRowName.ToString(), BaseProp->ArrayDim, PropertyValuesPtr->Num()));
			}

			for (int32 ArrayEntryIndex = 0; ArrayEntryIndex < BaseProp->ArrayDim; ++ArrayEntryIndex)
			{
				if (PropertyValuesPtr->IsValidIndex(ArrayEntryIndex))
				{
					void* Data = BaseProp->ContainerPtrToValuePtr<void>(InStructData, ArrayEntryIndex);
					const TSharedPtr<FJsonValue>& PropertyValueEntry = (*PropertyValuesPtr)[ArrayEntryIndex];
					ReadContainerEntry(PropertyValueEntry.ToSharedRef(), InRowName, ColumnName, ArrayEntryIndex, BaseProp, Data);
				}
			}
		}
	}

	return true;
}

bool FDataTableImporterJSON::ReadStructEntry(const TSharedRef<FJsonValue>& InParsedPropertyValue, const FName InRowName, const FString& InColumnName, const void* InRowData, FProperty* InProperty, void* InPropertyData)
{
	const TCHAR* const ParsedPropertyType = UE::DataTableJSON::JSONTypeToString(InParsedPropertyValue->Type);

	if (FEnumProperty* EnumProp = CastField<FEnumProperty>(InProperty))
	{
		FString EnumValue;
		if (InParsedPropertyValue->TryGetString(EnumValue))
		{
			FString Error = DataTableUtils::AssignStringToProperty(EnumValue, InProperty, (uint8*)InRowData);
			if (!Error.IsEmpty())
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' has invalid enum value: %s."), *InColumnName, *InRowName.ToString(), *EnumValue));
				return false;
			}
		}
		else
		{
			int64 PropertyValue = 0;
			if (!InParsedPropertyValue->TryGetNumber(PropertyValue))
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected Integer, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			EnumProp->GetUnderlyingProperty()->SetIntPropertyValue(InPropertyData, PropertyValue);
		}
	}
	else if (FNumericProperty *NumProp = CastField<FNumericProperty>(InProperty))
	{
		FString EnumValue;
		if (NumProp->IsEnum() && InParsedPropertyValue->TryGetString(EnumValue))
		{
			FString Error = DataTableUtils::AssignStringToProperty(EnumValue, InProperty, (uint8*)InRowData);
			if (!Error.IsEmpty())
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' has invalid enum value: %s."), *InColumnName, *InRowName.ToString(), *EnumValue));
				return false;
			}
		}
		else if (NumProp->IsInteger())
		{
			int64 PropertyValue = 0;
			if (!InParsedPropertyValue->TryGetNumber(PropertyValue))
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected Integer, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			NumProp->SetIntPropertyValue(InPropertyData, PropertyValue);
		}
		else
		{
			double PropertyValue = 0.0;
			if (!InParsedPropertyValue->TryGetNumber(PropertyValue))
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected Double, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			NumProp->SetFloatingPointPropertyValue(InPropertyData, PropertyValue);
		}
	}
	else if (FBoolProperty* BoolProp = CastField<FBoolProperty>(InProperty))
	{
		bool PropertyValue = false;
		if (!InParsedPropertyValue->TryGetBool(PropertyValue))
		{
			ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected Boolean, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
			return false;
		}

		BoolProp->SetPropertyValue(InPropertyData, PropertyValue);
	}
	else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(InProperty))
	{
		const TArray< TSharedPtr<FJsonValue> >* PropertyValuesPtr;
		if (!InParsedPropertyValue->TryGetArray(PropertyValuesPtr))
		{
			ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected Array, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
			return false;
		}

		FScriptArrayHelper ArrayHelper(ArrayProp, InPropertyData);
		ArrayHelper.EmptyValues();
		for (const TSharedPtr<FJsonValue>& PropertyValueEntry : *PropertyValuesPtr)
		{
			const int32 NewEntryIndex = ArrayHelper.AddValue();
			uint8* ArrayEntryData = ArrayHelper.GetRawPtr(NewEntryIndex);
			ReadContainerEntry(PropertyValueEntry.ToSharedRef(), InRowName, InColumnName, NewEntryIndex, ArrayProp->Inner, ArrayEntryData);
		}
	}
	else if (FSetProperty* SetProp = CastField<FSetProperty>(InProperty))
	{
		const TArray< TSharedPtr<FJsonValue> >* PropertyValuesPtr;
		if (!InParsedPropertyValue->TryGetArray(PropertyValuesPtr))
		{
			ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected Array, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
			return false;
		}

		FScriptSetHelper SetHelper(SetProp, InPropertyData);
		SetHelper.EmptyElements();
		for (const TSharedPtr<FJsonValue>& PropertyValueEntry : *PropertyValuesPtr)
		{
			const int32 NewEntryIndex = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
			uint8* SetEntryData = SetHelper.GetElementPtr(NewEntryIndex);
			ReadContainerEntry(PropertyValueEntry.ToSharedRef(), InRowName, InColumnName, NewEntryIndex, SetHelper.GetElementProperty(), SetEntryData);
		}
		SetHelper.Rehash();
	}
	else if (FMapProperty* MapProp = CastField<FMapProperty>(InProperty))
	{
		const TSharedPtr<FJsonObject>* PropertyValue;
		if (!InParsedPropertyValue->TryGetObject(PropertyValue))
		{
			ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected Object, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
			return false;
		}

		FScriptMapHelper MapHelper(MapProp, InPropertyData);
		MapHelper.EmptyValues();
		for (const auto& PropertyValuePair : (*PropertyValue)->Values)
		{
			const int32 NewEntryIndex = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
			uint8* MapKeyData = MapHelper.GetKeyPtr(NewEntryIndex);
			uint8* MapValueData = MapHelper.GetValuePtr(NewEntryIndex);

			// JSON object keys are always strings
			const FString KeyError = DataTableUtils::AssignStringToPropertyDirect(*PropertyValuePair.Key, MapHelper.GetKeyProperty(), MapKeyData);
			if (KeyError.Len() > 0)
			{
				MapHelper.RemoveAt(NewEntryIndex);
				ImportProblems.Add(FString::Printf(TEXT("Problem assigning key '%s' to property '%s' on row '%s' : %s"), *PropertyValuePair.Key, *InColumnName, *InRowName.ToString(), *KeyError));
				return false;
			}

			if (!ReadContainerEntry(PropertyValuePair.Value.ToSharedRef(), InRowName, InColumnName, NewEntryIndex, MapHelper.GetValueProperty(), MapValueData))
			{
				MapHelper.RemoveAt(NewEntryIndex);
				return false;
			}
		}
		MapHelper.Rehash();
	}
	else if (FStructProperty* StructProp = CastField<FStructProperty>(InProperty))
	{
		const TSharedPtr<FJsonObject>* PropertyValue = nullptr;
		if (InParsedPropertyValue->TryGetObject(PropertyValue))
		{
			return ReadStruct(PropertyValue->ToSharedRef(), StructProp->Struct, InRowName, InPropertyData);
		}
		else
		{
			// If the JSON does not contain a JSON object for this struct, we try to use the backwards-compatible string deserialization, same as the "else" block below
			FString PropertyValueString;
			if (!InParsedPropertyValue->TryGetString(PropertyValueString))
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected String, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			const FString Error = DataTableUtils::AssignStringToProperty(PropertyValueString, InProperty, (uint8*)InRowData);
			if (Error.Len() > 0)
			{
				ImportProblems.Add(FString::Printf(TEXT("Problem assigning string '%s' to property '%s' on row '%s' : %s"), *PropertyValueString, *InColumnName, *InRowName.ToString(), *Error));
				return false;
			}

			return true;
		}
	}
	else
	{
		FString PropertyValue;
		if (!InParsedPropertyValue->TryGetString(PropertyValue))
		{
			ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected String, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
			return false;
		}

		const FString Error = DataTableUtils::AssignStringToProperty(PropertyValue, InProperty, (uint8*)InRowData);
		if(Error.Len() > 0)
		{
			ImportProblems.Add(FString::Printf(TEXT("Problem assigning string '%s' to property '%s' on row '%s' : %s"), *PropertyValue, *InColumnName, *InRowName.ToString(), *Error));
			return false;
		}
	}

	return true;
}

bool FDataTableImporterJSON::ReadContainerEntry(const TSharedRef<FJsonValue>& InParsedPropertyValue, const FName InRowName, const FString& InColumnName, const int32 InArrayEntryIndex, FProperty* InProperty, void* InPropertyData)
{
	const TCHAR* const ParsedPropertyType = UE::DataTableJSON::JSONTypeToString(InParsedPropertyValue->Type);

	if (FEnumProperty* EnumProp = CastField<FEnumProperty>(InProperty))
	{
		FString EnumValue;
		if (InParsedPropertyValue->TryGetString(EnumValue))
		{
			FString Error = DataTableUtils::AssignStringToPropertyDirect(EnumValue, InProperty, (uint8*)InPropertyData);
			if (!Error.IsEmpty())
			{
				ImportProblems.Add(FString::Printf(TEXT("Entry %d on property '%s' on row '%s' has invalid enum value: %s."), InArrayEntryIndex, *InColumnName, *InRowName.ToString(), *EnumValue));
				return false;
			}
		}
		else
		{
			int64 PropertyValue = 0;
			if (!InParsedPropertyValue->TryGetNumber(PropertyValue))
			{
				ImportProblems.Add(FString::Printf(TEXT("Entry %d on property '%s' on row '%s' is the incorrect type. Expected Integer, got %s."), InArrayEntryIndex, *InColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			EnumProp->GetUnderlyingProperty()->SetIntPropertyValue(InPropertyData, PropertyValue);
		}
	}
	else if (FNumericProperty *NumProp = CastField<FNumericProperty>(InProperty))
	{
		FString EnumValue;
		if (NumProp->IsEnum() && InParsedPropertyValue->TryGetString(EnumValue))
		{
			FString Error = DataTableUtils::AssignStringToPropertyDirect(EnumValue, InProperty, (uint8*)InPropertyData);
			if (!Error.IsEmpty())
			{
				ImportProblems.Add(FString::Printf(TEXT("Entry %d on property '%s' on row '%s' has invalid enum value: %s."), InArrayEntryIndex, *InColumnName, *InRowName.ToString(), *EnumValue));
				return false;
			}
		}
		else if(NumProp->IsInteger())
		{
			int64 PropertyValue = 0;
			if (!InParsedPropertyValue->TryGetNumber(PropertyValue))
			{
				ImportProblems.Add(FString::Printf(TEXT("Entry %d on property '%s' on row '%s' is the incorrect type. Expected Integer, got %s."), InArrayEntryIndex, *InColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			NumProp->SetIntPropertyValue(InPropertyData, PropertyValue);
		}
		else
		{
			double PropertyValue = 0.0;
			if (!InParsedPropertyValue->TryGetNumber(PropertyValue))
			{
				ImportProblems.Add(FString::Printf(TEXT("Entry %d on property '%s' on row '%s' is the incorrect type. Expected Double, got %s."), InArrayEntryIndex, *InColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			NumProp->SetFloatingPointPropertyValue(InPropertyData, PropertyValue);
		}
	}
	else if (FBoolProperty* BoolProp = CastField<FBoolProperty>(InProperty))
	{
		bool PropertyValue = false;
		if (!InParsedPropertyValue->TryGetBool(PropertyValue))
		{
			ImportProblems.Add(FString::Printf(TEXT("Entry %d on property '%s' on row '%s' is the incorrect type. Expected Boolean, got %s."), InArrayEntryIndex, *InColumnName, *InRowName.ToString(), ParsedPropertyType));
			return false;
		}

		BoolProp->SetPropertyValue(InPropertyData, PropertyValue);
	}
	else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(InProperty))
	{
		// Cannot nest arrays
		return false;
	}
	else if (FSetProperty* SetProp = CastField<FSetProperty>(InProperty))
	{
		// Cannot nest sets
		return false;
	}
	else if (FMapProperty* MapProp = CastField<FMapProperty>(InProperty))
	{
		// Cannot nest maps
		return false;
	}
	else if (FStructProperty* StructProp = CastField<FStructProperty>(InProperty))
	{
		const TSharedPtr<FJsonObject>* PropertyValue = nullptr;
		if (InParsedPropertyValue->TryGetObject(PropertyValue))
		{
			return ReadStruct(PropertyValue->ToSharedRef(), StructProp->Struct, InRowName, InPropertyData);
		}
		else
		{
			// If the JSON does not contain a JSON object for this struct, we try to use the backwards-compatible string deserialization, same as the "else" block below
			FString PropertyValueString;
			if (!InParsedPropertyValue->TryGetString(PropertyValueString))
			{
				ImportProblems.Add(FString::Printf(TEXT("Property '%s' on row '%s' is the incorrect type. Expected String, got %s."), *InColumnName, *InRowName.ToString(), ParsedPropertyType));
				return false;
			}

			const FString Error = DataTableUtils::AssignStringToPropertyDirect(PropertyValueString, InProperty, (uint8*)InPropertyData);
			if (Error.Len() > 0)
			{
				ImportProblems.Add(FString::Printf(TEXT("Problem assigning string '%s' to entry %d on property '%s' on row '%s' : %s"), *PropertyValueString, InArrayEntryIndex, *InColumnName, *InRowName.ToString(), *Error));
				return false;
			}

			return true;
		}
	}
	else
	{
		FString PropertyValue;
		if (!InParsedPropertyValue->TryGetString(PropertyValue))
		{
			ImportProblems.Add(FString::Printf(TEXT("Entry %d on property '%s' on row '%s' is the incorrect type. Expected String, got %s."), InArrayEntryIndex, *InColumnName, *InRowName.ToString(), ParsedPropertyType));
			return false;
		}

		const FString Error = DataTableUtils::AssignStringToPropertyDirect(PropertyValue, InProperty, (uint8*)InPropertyData);
		if(Error.Len() > 0)
		{
			ImportProblems.Add(FString::Printf(TEXT("Problem assigning string '%s' to entry %d on property '%s' on row '%s' : %s"), *PropertyValue, InArrayEntryIndex, *InColumnName, *InRowName.ToString(), *Error));
			return false;
		}
	}

	return true;
}

