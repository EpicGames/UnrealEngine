// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"

class UDataTable;
enum class EDataTableExportFlags : uint8;
enum class EPropertyVisitorControlFlow : uint8;

template <class CharType> struct TPrettyJsonPrintPolicy;

// forward declare JSON writer
template <class CharType>
struct TPrettyJsonPrintPolicy;
template <class CharType, class PrintPolicy>
class TJsonWriter;

namespace DataTableJSONUtils
{
	/** Returns what string is used as the key/name field for a data table */
	FString GetKeyFieldName(const UDataTable& InDataTable);
}

#if WITH_EDITOR

/** 
 * Class to serialize a DataTable to a TJsonWriter.
 */
template<typename CharType = TCHAR>
class TDataTableExporterJSON
{
public:
	typedef TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType>> FDataTableJsonWriter;

	TDataTableExporterJSON(const EDataTableExportFlags InDTExportFlags, TSharedRef<FDataTableJsonWriter> InJsonWriter);

	~TDataTableExporterJSON();

	/** Writes the data table out as an array of objects */
	bool WriteTable(const UDataTable& InDataTable);

	/** Writes the data table out as a named object with each row being a sub value on that object */
	bool WriteTableAsObject(const UDataTable& InDataTable);

	/** Writes out a single row */
	bool WriteRow(const UScriptStruct* InRowStruct, const void* InRowData, const FString* FieldToSkip = nullptr);

	/** Writes the contents of a single row */
	bool WriteStruct(const UScriptStruct* InStruct, const void* InStructData, const FString* FieldToSkip = nullptr);

protected:
	//~ Field iterator based methods BEGIN
	bool WriteStructUsingFieldIterator(const UScriptStruct* InStruct, const void* InStructData, const FString* FieldToSkip = nullptr);
	bool WriteStructEntry(const void* InRowData, const FProperty* InProperty, const void* InPropertyData);
	bool WriteContainerEntry(const FProperty* InProperty, const void* InPropertyData, const FString* InIdentifier = nullptr);
	//~ Field iterator based methods END

	//~ PropertyVisitor based methods BEGIN
	bool WriteStructUsingVisitor(TNotNull<const UScriptStruct*> InStruct, const void* InStructData, const FString* FieldToSkip = nullptr);
	EPropertyVisitorControlFlow WriteVisitedEntry(TNotNull<const FProperty*> InProperty, const void* InPropertyData, const FString* InIdentifier = nullptr);
	//~ PropertyVisitor based methods END

	EDataTableExportFlags DTExportFlags;
	TSharedRef<FDataTableJsonWriter> JsonWriter;
	bool bJsonWriterNeedsClose;
};

/**
 * TCHAR-specific instantiation of TDataTableExporterJSON that has a convenience constructor to write output to an FString instead of an external TJsonWriter
 */
class FDataTableExporterJSON : public TDataTableExporterJSON<TCHAR>
{
public:
	using TDataTableExporterJSON<TCHAR>::TDataTableExporterJSON;
// 
	FDataTableExporterJSON(const EDataTableExportFlags InDTExportFlags, FString& OutExportText);
};

#endif // WITH_EDITOR

class FDataTableImporterJSON
{
public:
	FDataTableImporterJSON(UDataTable& InDataTable, const FString& InJSONData, TArray<FString>& OutProblems);

	~FDataTableImporterJSON();

	bool ReadTable();
	bool AddRow(bool bRemoveDuplicate = false, FName* OutRowName = nullptr);

private:
	bool ReadRow(const TSharedRef<FJsonObject>& InParsedTableRowObject, const int32 InRowIdx, bool bRemoveDuplicate = false, FName* OutRowName = nullptr);

	bool ReadStruct(const TSharedRef<FJsonObject>& InParsedObject, UScriptStruct* InStruct, const FName InRowName, void* InStructData);

	bool ReadStructEntry(const TSharedRef<FJsonValue>& InParsedPropertyValue, const FName InRowName, const FString& InColumnName, const void* InRowData, FProperty* InProperty, void* InPropertyData);

	bool ReadContainerEntry(const TSharedRef<FJsonValue>& InParsedPropertyValue, const FName InRowName, const FString& InColumnName, const int32 InArrayEntryIndex, FProperty* InProperty, void* InPropertyData);

	UDataTable* DataTable;
	const FString& JSONData;
	TArray<FString>& ImportProblems;
};
