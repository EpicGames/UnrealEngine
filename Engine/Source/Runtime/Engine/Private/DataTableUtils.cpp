// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableUtils.h"
#include "UObject/FieldPathProperty.h"
#include "UObject/TextProperty.h"
#include "Engine/DataTable.h"
#include "Engine/UserDefinedEnum.h"
#include "Misc/StringOutputDevice.h"
#include "Serialization/JsonWriter.h"
#include "UObject/EnumProperty.h"
#include "DataTableJSON.h"

DEFINE_LOG_CATEGORY(LogDataTable);

namespace DataTableUtilsImpl
{

FString GetSourceString(const FText& Text)
{
	if (const FString* SourceString = FTextInspector::GetSourceString(Text))
	{
		return *SourceString;
	}
	return Text.ToString();
}

void AssignStringToPropertyDirect(const FString& InString, const FProperty* InProp, uint8* InData, const int32 InPortFlags, FStringOutputDevice& OutImportError)
{
	auto DoImportText = [&](const FString& InStringToImport)
	{
		InProp->ImportText_Direct(*InStringToImport, InData, nullptr, InPortFlags, &OutImportError);
	};

	bool bNeedsImport = true;

	UEnum* Enum = nullptr;

	if (const FEnumProperty* EnumProp = CastField<const FEnumProperty>(InProp))
	{
		Enum = EnumProp->GetEnum();
	}
	else if (const FByteProperty* ByteProp = CastField<const FByteProperty>(InProp))
	{
		if (ByteProp->IsEnum())
		{
			Enum = ByteProp->GetIntPropertyEnum();
		}
	}

	if (Enum)
	{
		// Enum properties may use the friendly name in their import data, however the FPropertyByte::ImportText function will only accept the internal enum entry name
		// Detect if we're using a friendly name for an entry, and if so, try and map it to the correct internal name before performing the import
		const int32 EnumIndex = Enum->GetIndexByNameString(InString);
		if(EnumIndex == INDEX_NONE)
		{
			// Couldn't find a match for the name we were given, try and find a match using the friendly names
			for(int32 EnumEntryIndex = 0; EnumEntryIndex < Enum->NumEnums(); ++EnumEntryIndex)
			{
				const FText FriendlyEnumEntryName = Enum->GetDisplayNameTextByIndex(EnumEntryIndex);
				if ((FriendlyEnumEntryName.ToString() == InString) || (GetSourceString(FriendlyEnumEntryName) == InString))
				{
					// Get the corresponding internal name and warn the user that we're using this fallback if not a user-defined enum
					FString StringToImport = Enum->GetNameStringByIndex(EnumEntryIndex);
					if (!Enum->IsA<UUserDefinedEnum>())
					{
						UE_LOGF(LogDataTable, Warning, "Could not a find matching enum entry for '%ls', but did find a matching display name. Will import using the enum entry corresponding to that display name ('%ls')", *InString, *StringToImport);
					}
					DoImportText(StringToImport);
					bNeedsImport = false;
					break;
				}
			}
		}
	}

	if(bNeedsImport)
	{
		DoImportText(InString);
	}
}

void AssignStringToProperty(const FString& InString, const FProperty* InProp, uint8* InData, const int32 InIndex, const int32 InPortFlags, FStringOutputDevice& OutImportError)
{
	uint8* ValuePtr = InProp->ContainerPtrToValuePtr<uint8>(InData, InIndex);
	AssignStringToPropertyDirect(InString, InProp, ValuePtr, InPortFlags, OutImportError);
}

void GetPropertyValueAsStringDirect(const FProperty* InProp, const uint8* InData, const int32 InPortFlags, const EDataTableExportFlags InDTExportFlags, FString& OutString)
{
#if WITH_EDITOR
	if (InPortFlags & PPF_PropertyWindow)
	{
		auto ExportStructAsJson = [InDTExportFlags](const UScriptStruct* InStruct, const void* InStructData) -> FString
		{
			FString JsonOutputStr;
			{
				TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonOutputStr);

				JsonWriter->WriteObjectStart();
				FDataTableExporterJSON(InDTExportFlags, JsonWriter).WriteStruct(InStruct, InStructData);
				JsonWriter->WriteObjectEnd();

				JsonWriter->Close();
			}

			JsonOutputStr.ReplaceInline(TEXT("\t"), TEXT(""), ESearchCase::CaseSensitive);
			JsonOutputStr.ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
			JsonOutputStr.ReplaceInline(TEXT("\n"), TEXT(" "), ESearchCase::CaseSensitive);

			return JsonOutputStr;
		};

		const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(InProp);
		const FSetProperty* SetProp = CastField<const FSetProperty>(InProp);
		const FMapProperty* MapProp = CastField<const FMapProperty>(InProp);

		if (ArrayProp && ArrayProp->Inner->IsA<FStructProperty>() && EnumHasAnyFlags(InDTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
		{
			const FStructProperty* StructInner = CastFieldChecked<const FStructProperty>(ArrayProp->Inner);

			OutString.AppendChar('(');

			FScriptArrayHelper ArrayHelper(ArrayProp, InData);
			for (int32 ArrayEntryIndex = 0; ArrayEntryIndex < ArrayHelper.Num(); ++ArrayEntryIndex)
			{
				if (ArrayEntryIndex > 0)
				{
					OutString.AppendChar(',');
					OutString.AppendChar(' ');
				}

				const uint8* ArrayEntryData = ArrayHelper.GetRawPtr(ArrayEntryIndex);
				OutString.Append(ExportStructAsJson(StructInner->Struct, ArrayEntryData));
			}

			OutString.AppendChar(')');
			return;
		}
		else if (SetProp && SetProp->ElementProp->IsA<FStructProperty>() && EnumHasAnyFlags(InDTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
		{
			const FStructProperty* StructInner = CastFieldChecked<const FStructProperty>(SetProp->ElementProp);

			OutString.AppendChar('(');

			FScriptSetHelper SetHelper(SetProp, InData);
			for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
			{
					if (It.GetLogicalIndex() > 0)
					{
						OutString.AppendChar(',');
						OutString.AppendChar(' ');
					}

					const uint8* SetEntryData = SetHelper.GetElementPtr(It);
					OutString.Append(ExportStructAsJson(StructInner->Struct, SetEntryData));
			}

			OutString.AppendChar(')');
			return;
		}
		else if (MapProp)
		{
			OutString.AppendChar('(');

			FScriptMapHelper MapHelper(MapProp, InData);
			for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
			{
				if (It.GetLogicalIndex() > 0)
				{
					OutString.AppendChar(',');
					OutString.AppendChar(' ');
				}

				const uint8* MapKeyData = MapHelper.GetKeyPtr(It);
				const uint8* MapValueData = MapHelper.GetValuePtr(It);

				OutString.AppendChar('"');
				GetPropertyValueAsStringDirect(MapHelper.GetKeyProperty(), MapKeyData, InPortFlags, InDTExportFlags, OutString);
				OutString.AppendChar('"');

				OutString.Append(TEXT(" = "));

				if (MapHelper.GetValueProperty()->IsA<FStructProperty>() && EnumHasAnyFlags(InDTExportFlags, EDataTableExportFlags::UseJsonObjectsForStructs))
				{
					const FStructProperty* StructMapValue = CastFieldChecked<const FStructProperty>(MapHelper.GetValueProperty());
					OutString.Append(ExportStructAsJson(StructMapValue->Struct, MapValueData));
				}
				else
				{
					GetPropertyValueAsStringDirect(MapHelper.GetValueProperty(), MapValueData, InPortFlags, InDTExportFlags, OutString);
				}
			}

			OutString.AppendChar(')');
			return;
		}
		else if (const FStructProperty* StructProp = CastField<const FStructProperty>(InProp))
		{
			OutString.Append(ExportStructAsJson(StructProp->Struct, InData));
			return;
		}
	}
#endif // WITH_EDITOR

	if (EnumHasAnyFlags(InDTExportFlags, EDataTableExportFlags::UseSimpleText) && InProp->IsA<FTextProperty>())
	{
		const FTextProperty* TextProp = CastFieldChecked<const FTextProperty>(InProp);
		const FText& TextValue = TextProp->GetPropertyValue(InData);
		OutString.Append(TextValue.ToString());
		return;
	}

	InProp->ExportText_Direct(OutString, InData, InData, nullptr, InPortFlags);
}

void GetPropertyValueAsString(const FProperty* InProp, const uint8* InData, const int32 InIndex, const int32 InPortFlags, const EDataTableExportFlags InDTExportFlags, FString& OutString)
{
	const uint8* ValuePtr = InProp->ContainerPtrToValuePtr<uint8>(InData, InIndex);
	return GetPropertyValueAsStringDirect(InProp, ValuePtr, InPortFlags, InDTExportFlags, OutString);
}

}

FString DataTableUtils::AssignStringToPropertyDirect(const FString& InString, const FProperty* InProp, uint8* InData)
{
	FStringOutputDevice ImportError;
	if(InProp && IsSupportedTableProperty(InProp))
	{
		DataTableUtilsImpl::AssignStringToPropertyDirect(InString, InProp, InData, PPF_ExternalEditor, ImportError);
	}

	FString Error = ImportError;
	return Error;
}

FString DataTableUtils::AssignStringToProperty(const FString& InString, const FProperty* InProp, uint8* InData)
{
	FStringOutputDevice ImportError;
	if(InProp && IsSupportedTableProperty(InProp))
	{
		if(InProp->ArrayDim == 1)
		{
			DataTableUtilsImpl::AssignStringToProperty(InString, InProp, InData, 0, PPF_ExternalEditor, ImportError);
		}
		else
		{
			if(InString.Len() >= 2 && InString[0] == '(' && InString[InString.Len() - 1] == ')')
			{
				// Trim the ( and )
				FString StringToSplit = InString.Mid(1, InString.Len() - 2);

				TArray<FString> Values;
				StringToSplit.ParseIntoArray(Values, TEXT(","), 1);

				if (InProp->ArrayDim != Values.Num())
				{
					UE_LOGF(LogDataTable, Warning, "%ls - Array is %d elements large, but we have %d values to import", *InProp->GetName(), InProp->ArrayDim, Values.Num());
				}

				for (int32 Index = 0; Index < InProp->ArrayDim; ++Index)
				{
					if (Values.IsValidIndex(Index))
					{
						DataTableUtilsImpl::AssignStringToProperty(Values[Index], InProp, InData, Index, PPF_Delimited, ImportError);
					}
				}
			}
			else
			{
				UE_LOGF(LogDataTable, Warning, "%ls - Malformed array string. It must start with '(' and end with ')'", *InProp->GetName());
			}
		}
	}

	FString Error = ImportError;
	return Error;
}

FString DataTableUtils::GetPropertyValueAsStringDirect(const FProperty* InProp, const uint8* InData, const EDataTableExportFlags InDTExportFlags)
{
	FString Result;

	if(InProp && IsSupportedTableProperty(InProp))
	{
		DataTableUtilsImpl::GetPropertyValueAsStringDirect(InProp, InData, PPF_ExternalEditor, InDTExportFlags, Result);
	}

	return Result;
}

FString DataTableUtils::GetPropertyValueAsString(const FProperty* InProp, const uint8* InData, const EDataTableExportFlags InDTExportFlags)
{
	FString Result;

	if(InProp && IsSupportedTableProperty(InProp))
	{
		if(InProp->ArrayDim == 1)
		{
			DataTableUtilsImpl::GetPropertyValueAsString(InProp, InData, 0, PPF_ExternalEditor, InDTExportFlags, Result);
		}
		else
		{
			Result.AppendChar('(');

			for(int32 Index = 0; Index < InProp->ArrayDim; ++Index)
			{
				DataTableUtilsImpl::GetPropertyValueAsString(InProp, InData, Index, PPF_Delimited | PPF_ExternalEditor, InDTExportFlags, Result);

				if((Index + 1) < InProp->ArrayDim)
				{
					Result.AppendChar(',');
				}
			}

			Result.AppendChar(')');
		}
	}

	return Result;
}

FText DataTableUtils::GetPropertyValueAsTextDirect(const FProperty* InProp, const uint8* InData)
{
	FText Result;

	if(InProp && IsSupportedTableProperty(InProp))
	{
		FString ExportedString;
		DataTableUtilsImpl::GetPropertyValueAsStringDirect(InProp, InData, PPF_PropertyWindow, EDataTableExportFlags::UseJsonObjectsForStructs, ExportedString);

		Result = FText::FromString(MoveTemp(ExportedString));
	}

	return Result;
}

FText DataTableUtils::GetPropertyValueAsText(const FProperty* InProp, const uint8* InData)
{
	FText Result;

	if(InProp && IsSupportedTableProperty(InProp))
	{
		FString ExportedString;

		if(InProp->ArrayDim == 1)
		{
			DataTableUtilsImpl::GetPropertyValueAsString(InProp, InData, 0, PPF_PropertyWindow, EDataTableExportFlags::UseJsonObjectsForStructs, ExportedString);
		}
		else
		{
			ExportedString.AppendChar('(');

			for(int32 Index = 0; Index < InProp->ArrayDim; ++Index)
			{
				DataTableUtilsImpl::GetPropertyValueAsString(InProp, InData, Index, PPF_PropertyWindow | PPF_Delimited, EDataTableExportFlags::UseJsonObjectsForStructs, ExportedString);

				if((Index + 1) < InProp->ArrayDim)
				{
					ExportedString.AppendChar(',');
					ExportedString.AppendChar(' ');
				}
			}

			ExportedString.AppendChar(')');
		}
		
		Result = FText::FromString(MoveTemp(ExportedString));
	}

	return Result;
}

TArray<FName> DataTableUtils::GetStructPropertyNames(const UStruct* InStruct)
{
	TArray<FName> PropNames;
	for (TFieldIterator<const FProperty> It(InStruct); It; ++It)
	{
		PropNames.Add(It->GetFName());
	}
	return PropNames;
}

FName DataTableUtils::MakeValidName(const FString& InString)
{
	FString InvalidChars(INVALID_NAME_CHARACTERS);

	FString FixedString;
	TArray<TCHAR, FString::AllocatorType>& FixedCharArray = FixedString.GetCharArray();

	// Iterate over input string characters
	for(int32 CharIdx=0; CharIdx<InString.Len(); CharIdx++)
	{
		// See if this char occurs in the InvalidChars string
		FString Char = InString.Mid( CharIdx, 1 );
		if( !InvalidChars.Contains(Char) )
		{
			// Its ok, add to result
			FixedCharArray.Add(Char[0]);
		}
	}
	FixedCharArray.Add(0);

	return FName(*FixedString);
}

bool DataTableUtils::IsSupportedTableProperty(const FProperty* InProp)
{
	return( InProp &&
			(InProp->IsA(FIntProperty::StaticClass()) || 
			InProp->IsA(FNumericProperty::StaticClass()) ||
			InProp->IsA(FDoubleProperty::StaticClass()) ||
			InProp->IsA(FFloatProperty::StaticClass()) ||
			InProp->IsA(FNameProperty::StaticClass()) ||
			InProp->IsA(FStrProperty::StaticClass()) ||
			InProp->IsA(FBoolProperty::StaticClass()) ||
			InProp->IsA(FObjectPropertyBase::StaticClass()) ||
			InProp->IsA(FStructProperty::StaticClass()) ||
			InProp->IsA(FByteProperty::StaticClass()) ||
			InProp->IsA(FTextProperty::StaticClass()) ||
			InProp->IsA(FArrayProperty::StaticClass()) ||
			InProp->IsA(FSetProperty::StaticClass()) ||
			InProp->IsA(FMapProperty::StaticClass()) ||
			InProp->IsA(FEnumProperty::StaticClass()) ||
			InProp->IsA(FFieldPathProperty::StaticClass())
			)
		
		);
}

FString DataTableUtils::GetPropertyExportName(const FProperty* Prop, const EDataTableExportFlags InDTExportFlags)
{
	if (!ensure(Prop))
	{
		return FString();
	}
	return Prop->GetAuthoredName();
}

TArray<FString> DataTableUtils::GetPropertyImportNames(const FProperty* Prop)
{
	TArray<FString> Result;
	GetPropertyImportNames(Prop, Result);
	return Result;
}

void DataTableUtils::GetPropertyImportNames(const FProperty* Prop, TArray<FString>& OutResult)
{
	OutResult.Reset();
	if (ensure(Prop))
	{
		OutResult.AddUnique(Prop->GetName());
	}
	OutResult.AddUnique(GetPropertyExportName(Prop));
}

FText DataTableUtils::GetPropertyDisplayName(const FProperty* Prop, const FString& DefaultName)
{
#if WITH_EDITOR
	if (Prop)
	{
		return Prop->GetDisplayNameText();
	}
#endif // WITH_EDITOR
	return FText::FromString(DefaultName);
}

TArray<FString> DataTableUtils::GetColumnDataAsString(const UDataTable* InTable, const FName& PropertyName, const EDataTableExportFlags InDTExportFlags)
{
	TArray<FString> Result;
	if (!ensure(InTable))
	{
		return Result;
	}
	if (!ensure(PropertyName != NAME_None))
	{
		return Result;
	}

	FProperty* ColumnProperty = InTable->FindTableProperty(PropertyName);
	if (ColumnProperty)
	{
		for (auto RowIt = InTable->GetRowMap().CreateConstIterator(); RowIt; ++RowIt)
		{
			uint8* RowData = RowIt.Value();
			Result.Add(GetPropertyValueAsString(ColumnProperty, RowData, InDTExportFlags));
		}
	}

	return Result;
}

bool DataTableUtils::AddRowJSON(UDataTable& InDataTable, const FString& InJSONData, TArray<FString>& OutProblems, bool bRemoveDuplicate, FName* OutRowName)
{
	return FDataTableImporterJSON(InDataTable, InJSONData, OutProblems).AddRow(bRemoveDuplicate, OutRowName);
}