// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/DebugDrawComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugDrawComponent)

FPrimitiveSceneProxy* UDebugDrawComponent::CreateSceneProxy()
{
	FDebugRenderSceneProxy* Proxy = CreateDebugSceneProxy();
  	if (Proxy != nullptr)
	{
		GetDebugDrawDelegateHelper().InitDelegateHelper(Proxy);
	}

	GetDebugDrawDelegateHelper().ProcessDeferredRegister();
	return Proxy;
}

  void UDebugDrawComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);
	GetDebugDrawDelegateHelper().RequestRegisterDebugDrawDelegate(Context);
}

void UDebugDrawComponent::DestroyRenderState_Concurrent()
{
	GetDebugDrawDelegateHelper().UnregisterDebugDrawDelegate();
	Super::DestroyRenderState_Concurrent();
}

