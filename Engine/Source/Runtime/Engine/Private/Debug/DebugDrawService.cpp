// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/DebugDrawService.h"
#include "SceneView.h"
#include "UObject/Package.h"
#include "Engine/Canvas.h"
#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugDrawService)

FTransactionallySafeCriticalSection UDebugDrawService::DelegatesLock;

TArray<UDebugDrawService::FDebugDrawMulticastDelegate> UDebugDrawService::Delegates;
FEngineShowFlags UDebugDrawService::ObservedFlags(ESFIM_Editor);

UDebugDrawService::UDebugDrawService(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Delegates.Reserve(sizeof(FEngineShowFlags)*8);
}

FDelegateHandle UDebugDrawService::Register(const TCHAR* Name, const FDebugDrawDelegate& NewDelegate)
{
	const int32 Index = FEngineShowFlags::FindIndexByName(Name);

	FDelegateHandle Result;
	if (Index != INDEX_NONE)
	{
		UE::TScopeLock ScopeLock(DelegatesLock);
		if (Index >= Delegates.Num())
		{
			Delegates.AddZeroed(Index - Delegates.Num() + 1);
		}
		Result = Delegates[Index].Add(NewDelegate);
		ObservedFlags.SetSingleFlag(Index, true);
	}
	return Result;
}

void UDebugDrawService::Unregister(const FDelegateHandle HandleToRemove)
{
	UE::TScopeLock ScopeLock(DelegatesLock);
	for (int32 Flag = 0; Flag < Delegates.Num(); ++Flag)
	{
		FDebugDrawMulticastDelegate& MulticastDelegate = Delegates[Flag];
		if (MulticastDelegate.Remove(HandleToRemove))
		{
			if (MulticastDelegate.IsBound() == false)
			{
				ObservedFlags.SetSingleFlag(Flag, false);
			}

			break;
		}
	}	
}

void UDebugDrawService::Draw(const FEngineShowFlags Flags, FViewport* Viewport, FSceneView* View, FCanvas* Canvas, UCanvas* CanvasObject, APlayerController* PlayerController)
{
	if (CanvasObject == nullptr)
	{
		CanvasObject = FindObject<UCanvas>(GetTransientPackage(), TEXT("DebugCanvasObject"));
		if (CanvasObject == nullptr)
		{
			CanvasObject = NewObject<UCanvas>(GetTransientPackage(), TEXT("DebugCanvasObject"));
			CanvasObject->AddToRoot();
		}
	}

	// Canvas must be initialized every draw because the FCanvas passed in is on the stack in some scenarios.
	CanvasObject->Init(View->UnscaledViewRect.Width(), View->UnscaledViewRect.Height(), View, Canvas);

	CanvasObject->Update();
	CanvasObject->SetView(View);

	// PreRender the player's view.
	Draw(Flags, CanvasObject, PlayerController);
}

void UDebugDrawService::Draw(const FEngineShowFlags Flags, UCanvas* Canvas, APlayerController* PlayerController)
{
	if (Canvas == nullptr || Canvas->Canvas == nullptr)
	{
		return;
	}

	UE::TScopeLock ScopeLock(DelegatesLock);
	for (int32 FlagIndex = 0; FlagIndex < Delegates.Num(); ++FlagIndex)
	{
		FDebugDrawMulticastDelegate& MulticastDelegate = Delegates[FlagIndex];

		if (Flags.GetSingleFlag(FlagIndex) && ObservedFlags.GetSingleFlag(FlagIndex))
		{
			MulticastDelegate.Broadcast(Canvas, PlayerController);
		}
	}
}
