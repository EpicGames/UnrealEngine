// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/ReporterBase.h"
#include "Engine/Canvas.h"
#include "GlobalRenderResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReporterBase)

#define DASH_LINE_SIZE	5.0f

UReporterBase::UReporterBase(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	bVisible = false;
}

FVector2D UReporterBase::ToScreenSpace(const FVector2D& InVector, UCanvas* Canvas)
{
	const float DPIScale = Canvas->GetDPIScale();

	FVector2D OutVector = InVector;
	OutVector.X *= (Canvas->SizeX / DPIScale);
	OutVector.Y *= (Canvas->SizeY / DPIScale);

	return OutVector;
}

void UReporterBase::DrawLine(UCanvas* Canvas,const FVector2D& StartPos,const FVector2D& EndPos,const FLinearColor& Color, EReporterLineStyle::Type LineStyle)
{
	FCanvasLineItem LineItem;
	LineItem.SetColor( Color );
	switch(LineStyle)
	{
		case EReporterLineStyle::Line:
		{
			LineItem.Draw( Canvas->Canvas, ToScreenSpace(StartPos, Canvas), ToScreenSpace(EndPos, Canvas) );			
		} break;

		case EReporterLineStyle::Dash:
		{
			float NormalizedDashSize = DASH_LINE_SIZE / Canvas->SizeX;
			FVector2D Dir = EndPos - StartPos;
			float LineLength = Dir.Size();
			Dir.Normalize();
			FVector2D CurrentLinePos = StartPos;

			while(FVector2D::DotProduct(EndPos - CurrentLinePos, Dir) > 0)
			{
				LineItem.Draw( Canvas->Canvas, ToScreenSpace(CurrentLinePos, Canvas), ToScreenSpace(CurrentLinePos + Dir * NormalizedDashSize, Canvas) );				
				CurrentLinePos +=  Dir * NormalizedDashSize * 2.0f;
			}
		} break;
	}
	
}

void UReporterBase::DrawTriangle(UCanvas* Canvas, const FVector2D& Vertex1, const FVector2D& Vertex2, const FVector2D& Vertex3, const FLinearColor& Color)
{
	FCanvasTriangleItem TriItem( ToScreenSpace(Vertex1, Canvas), ToScreenSpace(Vertex2, Canvas), ToScreenSpace(Vertex3, Canvas ), GWhiteTexture );
	TriItem.SetColor( Color );
	Canvas->DrawItem( TriItem );
}

