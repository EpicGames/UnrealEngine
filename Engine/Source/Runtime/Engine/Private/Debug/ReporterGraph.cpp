// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/ReporterGraph.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "GlobalRenderResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReporterGraph)

UReporterGraph::UReporterGraph(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	AxesColor = FLinearColor::Yellow;

	AxisStyle = EGraphAxisStyle::Grid;
	DataStyle = EGraphDataStyle::Filled;
	LegendPosition = ELegendPosition::Outside;
	NumXNotches = 10;
	NumYNotches = 10;
	bOffsetDataSets = false;
	UseTinyFont(false);
	DrawCursorOnGraph(false);

	BackgroundColor = FColor(0, 0, 0, 0);
}

void UReporterGraph::SetGraphScreenSize(float MinX, float MaxX, float MinY, float MaxY)
{
	FVector2D Min(MinX, MinY);
	FVector2D Max(MaxX, MaxY);

	SetGraphScreenSize(Min, Max);
}

void UReporterGraph::SetGraphScreenSize(const FVector2D& Min, const FVector2D& Max)
{
	GraphScreenSize.Min = Min;
	GraphScreenSize.Max = Max;
}

void UReporterGraph::Draw(UCanvas* Canvas)
{
	if(!bVisible)
	{
		return;
	}

	DrawBackground(Canvas);

	switch(DataStyle)
	{
		case EGraphDataStyle::Lines:
		{
			// order doesn't *really* matter, as they're lines
			DrawAxes(Canvas);
			DrawData(Canvas);
		} break;

		case EGraphDataStyle::Filled:
		{
			// draw data first and overlay axes
			DrawData(Canvas);
			DrawAxes(Canvas);
		} break;
	}

	DrawLegend(Canvas);

	DrawThresholds(Canvas);
}

void UReporterGraph::SetAxesMinMax(float MinX, float MaxX, float MinY, float MaxY)
{
	FVector2D Min(MinX, MinY);
	FVector2D Max(MaxX, MaxY);

	SetAxesMinMax(Min, Max);
}

void UReporterGraph::SetAxesMinMax(const FVector2D& Min, const FVector2D& Max)
{
	GraphMinMaxData.Min = Min;
	GraphMinMaxData.Max = Max;
}

void UReporterGraph::SetNumGraphLines(int32 NumDataLines)
{
	CurrentData.Empty();

	FGraphLine NewDataLine;

	for(int32 i = 0; i < NumDataLines; i++)
	{
		CurrentData.Add(NewDataLine);
	}

	LegendWidth = MIN_flt;
}

FGraphLine* UReporterGraph::GetGraphLine(int32 LineIndex)
{
	if(LineIndex < 0 || LineIndex > CurrentData.Num())
	{
		return NULL;
	}

	return &CurrentData[LineIndex];
}

void UReporterGraph::SetNumThresholds(int32 NumThresholds)
{
	Thresholds.Empty();

	FGraphThreshold NewThreshold;

	for(int32 i = 0; i < NumThresholds; i++)
	{
		Thresholds.Add(NewThreshold);
	}

	LegendWidth = MIN_flt;
}

FGraphThreshold* UReporterGraph::GetThreshold(int32 ThresholdIndex)
{
	if (!Thresholds.IsValidIndex(ThresholdIndex))
	{
		return NULL;
	}

	return &Thresholds[ThresholdIndex];
}

void UReporterGraph::SetBackgroundColor(FColor Color)
{
	BackgroundColor = Color;
}

void UReporterGraph::SetLegendPosition(ELegendPosition::Type Position)
{
	LegendPosition = Position;
}

void UReporterGraph::DrawBackground(UCanvas* Canvas)
{
	FVector2D Min = ToScreenSpace(GraphScreenSize.Min, Canvas);
	FVector2D Max = ToScreenSpace(GraphScreenSize.Max, Canvas);

	FCanvasTileItem TileItem(Min, GWhiteTexture, Max-Min, BackgroundColor);
	TileItem.BlendMode = SE_BLEND_Translucent;
	Canvas->DrawItem(TileItem, Min);
}

void UReporterGraph::DrawLegend(UCanvas* Canvas)
{
	FVector2D CurrentTextPos = GraphScreenSize.Max;
	CurrentTextPos.X = GraphScreenSize.Min.X;
	UFont* Font = GetDefaultFont();

	int32 DummyY, CurrentX;
	StringSize(Font, CurrentX, DummyY, TEXT("99.99"));
	const float InsideLegendWidth = -CurrentX;

	for(int32 i = 0; i < CurrentData.Num(); i++)
	{
		FVector2D ScreenPos = ToScreenSpace(CurrentTextPos, Canvas);
		if (LegendPosition == ELegendPosition::Outside)
		{
			StringSize(Font, CurrentX, DummyY, *CurrentData[i].LineName);
			LegendWidth = CurrentX + 10;
		}
		else
		{
			LegendWidth = InsideLegendWidth;
		}

		FCanvasTextItem TextItem(FVector2D::ZeroVector, FText::FromString(*CurrentData[i].LineName), Font, CurrentData[i].Color);
		TextItem.EnableShadow(FColor::Black, FVector2D(1, 1));
		TextItem.SetColor(CurrentData[i].Color);
		Canvas->DrawItem(TextItem, ScreenPos.X - LegendWidth, ScreenPos.Y);

		CurrentTextPos.Y -= (GraphScreenSize.Max.Y - GraphScreenSize.Min.Y) / CurrentData.Num();
	}

	
}

void UReporterGraph::DrawAxes(UCanvas* Canvas)
{
	const float DPIScale = Canvas->GetDPIScale();
	FVector2D Min = GraphScreenSize.Min;
	
	FVector2D XMax = GraphScreenSize.Min;
	XMax.X = GraphScreenSize.Max.X;
	
	FVector2D YMax = GraphScreenSize.Min;
	YMax.Y = GraphScreenSize.Max.Y;

	UFont* Font = GetDefaultFont();
	int32 StringSizeX, StringSizeY;
	// Draw the X axis
	StringSize(Font, StringSizeX, StringSizeY, *FString::Printf(TEXT("%.2f"), GraphMinMaxData.Max.X) );
	const float SizeX = ((XMax.X - Min.X) * Canvas->SizeX) / DPIScale;
	NumXNotches = FMath::CeilToInt(SizeX * 0.7 / StringSizeX);
	DrawAxis(Canvas, Min, XMax, NumXNotches, false);

	// Draw the Y axis
	StringSize(Font, StringSizeX, StringSizeY, *FString::Printf(TEXT("%.2f"), GraphMinMaxData.Max.Y));
	float SizeY = ((YMax.Y - Min.Y) * Canvas->SizeY) / DPIScale;
	NumYNotches = FMath::CeilToInt(SizeY * 0.7 / StringSizeY);
	DrawAxis(Canvas, Min, YMax, NumYNotches, true);
}

void UReporterGraph::DrawAxis(UCanvas* Canvas, FVector2D Start, FVector2D End, float NumNotches, bool bIsVerticalAxis)
{
	// Draw the axis line
	DrawLine(Canvas, Start, End, AxesColor);

	// Divide each axis up into the desired notches
	float NotchDelta = (Start - End).Size() / (float)NumNotches;
	FVector2D NotchDataDelta = (GraphMinMaxData.Max - GraphMinMaxData.Min) / NumNotches;
	FVector2D NotchLocation = Start;
	FVector2D NotchLength(0,0);

	FLinearColor NotchColor = AxesColor;

	// if we should just draw notches
	switch(AxisStyle)
	{
		case EGraphAxisStyle::Lines:
		{
			NumNotches = 1;
			NotchDelta = (Start - End).Size();
			if(bIsVerticalAxis)
			{
				NotchLength.X = -(End.Y - Start.Y) * 0.05f;
				NotchLocation.X += NotchLength.X * -0.5f;
			}
			else
			{
				NotchLength.Y = -(End.X - Start.X) * 0.05f;
				NotchLocation.Y += NotchLength.Y * -0.5f;
			}
		} break;

		case EGraphAxisStyle::Notches:
		{
			if(bIsVerticalAxis)
			{
				NotchLength.X = -(End.Y - Start.Y) * 0.05f;
				NotchLocation.X += NotchLength.X * -0.5f;
			}
			else
			{
				NotchLength.Y = -(End.X - Start.X) * 0.05f;
				NotchLocation.Y += NotchLength.Y * -0.5f;
			}
		} break;

		case EGraphAxisStyle::Grid:
		{
			NotchColor *= 0.125f;

			if(bIsVerticalAxis)
			{
				NotchLength.X = End.Y - Start.Y;
			}
			else
			{
				NotchLength.Y = End.X - Start.X;
			}
		} break;
	}
	
	UFont* Font = GetDefaultFont();
	const FVector2D Width = FVector2D((GraphScreenSize.Max.X - GraphScreenSize.Min.X), 0);
	const FVector2D Height = FVector2D(0, (GraphScreenSize.Max.Y - GraphScreenSize.Min.Y));

	for (int Index = 0; Index < NumNotches + 1; Index++)
	{
		FString NotchValue = FString::Printf(TEXT("%1.2f"), (bIsVerticalAxis ? GraphMinMaxData.Min.Y + (NotchDataDelta.Y * (Index /*+ 1*/)) : GraphMinMaxData.Min.X + (NotchDataDelta.X * (Index /*+ 1*/))));
		
		int32 StringSizeX, StringSizeY;
		StringSize(Font, StringSizeX, StringSizeY, *NotchValue );

		FVector2D ScreenPos = ToScreenSpace(NotchLocation, Canvas);

		if (bIsVerticalAxis)
		{
			Canvas->Canvas->DrawShadowedString(ScreenPos.X - StringSizeX - 4, ScreenPos.Y - StringSizeY * 0.5f, *NotchValue, Font, AxesColor);
			DrawLine(Canvas, NotchLocation, NotchLocation + Width, NotchColor);
		}
		else
		{
			Canvas->Canvas->DrawShadowedString( ScreenPos.X - StringSizeX * 0.5f, ScreenPos.Y + (AxisStyle == EGraphAxisStyle::Grid ? + 5 : -NotchLength.Y * Canvas->SizeY), *NotchValue , Font, AxesColor);
			DrawLine(Canvas, NotchLocation, NotchLocation + Height, NotchColor);
		}
		
		if(bIsVerticalAxis)
		{
			NotchLocation.Y += NotchDelta;
		}
		else
		{
			NotchLocation.X += NotchDelta;
		}
	}

	if (bIsVerticalAxis && bDrawExtremes)
	{
		for (int32 i = 0; i < CurrentData.Num(); i++)
		{

			if (CurrentData[i].Data.Num() == 0)
				continue;

			FVector2D DataStart = CurrentData[i].LeftExtreme;
			DataStart.X = Start.X;
			FVector2D TextPos = ToScreenSpace(DataToNormalized(DataStart), Canvas) /*+ DrawOffset*/;

			FString Text = FString::Printf(TEXT("%.2f"), CurrentData[i].LeftExtreme.Y);
			int32 StringSizeX, StringSizeY;
			StringSize(Font, StringSizeX, StringSizeY, *Text);
			Canvas->Canvas->DrawShadowedString(TextPos.X - StringSizeX * 0.5f, TextPos.Y + (AxisStyle == EGraphAxisStyle::Grid ? +5 : -NotchLength.Y * Canvas->SizeY), *Text, Font, CurrentData[i].Color);

			FVector2D DataEnd = CurrentData[i].Data[CurrentData[i].Data.Num()-1];
			DataEnd = DataToNormalized(DataEnd);
			DataEnd.X = GraphScreenSize.Max.X;
			TextPos = ToScreenSpace(DataEnd, Canvas) /*+ DrawOffset*/;

			Text = FString::Printf(TEXT("%.2f"), CurrentData[i].RightExtreme.Y);
			StringSize(Font, StringSizeX, StringSizeY, *Text);
			Canvas->Canvas->DrawShadowedString(TextPos.X - StringSizeX * 0.5f, TextPos.Y + (AxisStyle == EGraphAxisStyle::Grid ? +5 : -NotchLength.Y * Canvas->SizeY), *Text, Font, CurrentData[i].Color);
		}
	}
}

void UReporterGraph::DrawThresholds(UCanvas* Canvas)
{
	UFont* Font = GetDefaultFont();
	for(int32 i = 0; i < Thresholds.Num(); i++)
	{
		if(Thresholds[i].Threshold < GraphMinMaxData.Max.Y)
		{
			FVector2D ThresholdStart(0, Thresholds[i].Threshold);
			ThresholdStart = DataToNormalized(ThresholdStart);

			FVector2D ThresholdEnd = ThresholdStart;
			ThresholdEnd.X = GraphScreenSize.Max.X;
		
			DrawLine(Canvas, ThresholdStart, ThresholdEnd, Thresholds[i].Color, EReporterLineStyle::Dash);
			FVector2D TextPos = ToScreenSpace(ThresholdEnd, Canvas);
			Canvas->Canvas->DrawShadowedString( TextPos.X, TextPos.Y, *Thresholds[i].ThresholdName , Font, Thresholds[i].Color);
		}
		else
		{
			break;
		}
	}
}

void UReporterGraph::DrawData(UCanvas* Canvas)
{
	FVector2D Start, End;
	
	const FVector2D Min = ToScreenSpace(GraphScreenSize.Min, Canvas);
	const FVector2D Max = ToScreenSpace(GraphScreenSize.Max, Canvas);
	const float Height = GraphScreenSize.Max.Y - GraphScreenSize.Min.Y;
	const float dx = Height / FMath::Abs(Max.Y - Min.Y);
	UFont* Font = GetDefaultFont();
	int32 StringSizeX, StringSizeY;
	// Draw the X axis
	StringSize(Font, StringSizeX, StringSizeY, TEXT("0"));

	float UpOffset = 0;
	float DownOffset = 0;

	if (bDrawCursorOnGraph && DataStyle == EGraphDataStyle::Lines)
	{
		DrawLine(Canvas, DataToNormalized(FVector2D(CursorLocation, GraphMinMaxData.Min.Y)), DataToNormalized(FVector2D(CursorLocation, GraphMinMaxData.Max.Y)), FLinearColor::White, EReporterLineStyle::Line);
	}

	for (int32 i = 0; i < CurrentData.Num(); i++)
	{
		if (IsOffsetForDataSetsEnabled())
		{
			if (i % 2)
			{
				UpOffset += dx;
			}
			else
			{
				DownOffset += dx;
			}
		}

		for (int32 j = 1; j < CurrentData[i].Data.Num(); j++)
		{
			FVector2D DataStart = Start = CurrentData[i].Data[j - 1];
			FVector2D DataEnd = End = CurrentData[i].Data[j];

			Start = DataToNormalized(Start);
			End = DataToNormalized(End);

			if (DataStyle == EGraphDataStyle::Lines)
			{
				const FVector2D DrawOffset = FVector2D(UpOffset, UpOffset) * ((i % 2) ? 1.0 : -1.0f);
				DrawLine(Canvas, Start + DrawOffset, End + DrawOffset, CurrentData[i].Color);

				if (bDrawCursorOnGraph && CursorLocation >= DataStart.X && CursorLocation < DataEnd.X)
				{
					const float t = (CursorLocation - DataStart.X) / (DataEnd.X - DataStart.X);
					FVector2D Location = FMath::Lerp<FVector2D, float>(DataStart, DataEnd, t);

					FVector2D TextPos = ToScreenSpace(DataToNormalized(Location), Canvas) + DrawOffset;
					Canvas->Canvas->DrawShadowedString(StringSizeX + TextPos.X, TextPos.Y, *FString::Printf(TEXT("%1.2f"), Location.Y), Font, CurrentData[i].Color);
				}
			}
			else
			{
				FVector2D Position0, Position1, Position2;

				// draw the top triangle of the quad
				Position0.X = Start.X;
				Position0.Y = (GraphMinMaxData.Min.Y * (GraphScreenSize.Max.Y - GraphScreenSize.Min.Y)) + GraphScreenSize.Min.Y;
				Position1 = End;
				Position2 = Start;
				DrawTriangle(Canvas, Position0, Position1, Position2, CurrentData[i].Color);


				// draw the second triangle of the quad
				Position0.X = Start.X;
				Position0.Y = (GraphMinMaxData.Min.Y * (GraphScreenSize.Max.Y - GraphScreenSize.Min.Y)) + GraphScreenSize.Min.Y;
				Position1.X = End.X;
				Position1.Y = (GraphMinMaxData.Min.Y * (GraphScreenSize.Max.Y - GraphScreenSize.Min.Y)) + GraphScreenSize.Min.Y;
				Position2 = End;

				DrawTriangle(Canvas, Position0, Position1, Position2, CurrentData[i].Color);
			}
		}
	}
}

FVector2D UReporterGraph::ToScreenSpace(const FVector2D& InVector, UCanvas* Canvas)
{
	const float DPIScale = Canvas->GetDPIScale();

	FVector2D OutVector = InVector;
	OutVector.X = (OutVector.X * Canvas->SizeX) / DPIScale;
	OutVector.Y = (Canvas->SizeY - (OutVector.Y * Canvas->SizeY)) / DPIScale;

	return OutVector;
}

FVector2D UReporterGraph::DataToNormalized(const FVector2D& InVector)
{
	FVector2D OutVector = InVector;
	OutVector.X = FMath::Clamp((OutVector.X - GraphMinMaxData.Min.X) / (GraphMinMaxData.Max.X - GraphMinMaxData.Min.X), 0.0f, 1.0f);
	OutVector.Y = FMath::Clamp((OutVector.Y - GraphMinMaxData.Min.Y) / (GraphMinMaxData.Max.Y - GraphMinMaxData.Min.Y), 0.0f, 1.0f);

	OutVector.X = (OutVector.X * (GraphScreenSize.Max.X - GraphScreenSize.Min.X)) + GraphScreenSize.Min.X;
	OutVector.Y = (OutVector.Y * (GraphScreenSize.Max.Y - GraphScreenSize.Min.Y)) + GraphScreenSize.Min.Y;
	
	return OutVector;
}
					   
UFont* UReporterGraph::GetDefaultFont()
{
	if (bUseTinyFont)
	{
		return GEngine->GetTinyFont();
	}

	return GEngine->GetSmallFont();
}

