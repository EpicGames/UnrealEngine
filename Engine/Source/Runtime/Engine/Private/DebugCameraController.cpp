// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
   DebugCameraController.cpp: Native implementation for the debug camera

=============================================================================*/

#include "Engine/DebugCameraController.h"
#include "Components/MeshComponent.h"
#include "Engine/DebugCameraControllerSettings.h"
#include "Components/InputComponent.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Player.h"
#include "FinalPostProcessSettings.h"
#include "ShaderCore.h"
#include "EngineUtils.h"
#include "GameFramework/SpectatorPawn.h"
#include "GameFramework/SpectatorPawnMovement.h"
#include "Engine/DebugCameraHUD.h"
#include "Components/DrawFrustumComponent.h"
#include "GameFramework/PlayerInput.h"
#include "GameFramework/GameStateBase.h"
#include "BufferVisualizationData.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugCameraController)

static const float SPEED_SCALE_ADJUSTMENT = 0.05f;
static const float MIN_ORBIT_RADIUS = 30.0f;

ADebugCameraController::ADebugCameraController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	OriginalControllerRef = nullptr;
	OriginalPlayer = nullptr;

	SpeedScale = 1.f;
	InitialMaxSpeed = 0.f;
	InitialAccel = 0.f;
	InitialDecel = 0.f;

	bIsFrozenRendering = false;
	DrawFrustum = nullptr;
	SetHidden(false);
#if WITH_EDITORONLY_DATA
	bHiddenEd = false;
#endif // WITH_EDITORONLY_DATA
	PrimaryActorTick.bTickEvenWhenPaused = true;
	bShouldPerformFullTickWhenPaused = true;
	SetAsLocalPlayerController();

	bIsOrbitingSelectedActor = false;
	bOrbitPivotUseCenter = false;
	LastOrbitPawnLocation = FVector::ZeroVector;
	OrbitPivot = FVector::ZeroVector;
	OrbitRadius = MIN_ORBIT_RADIUS;

	bEnableBufferVisualization = false;
	bEnableBufferVisualizationFullMode = false;
	bIsBufferVisualizationInputSetup = false;
	bLastDisplayEnabled = true;
	LastViewModeSettingsIndex = 0;
}

void InitializeDebugCameraInputBindings()
{
	static bool bBindingsAdded = false;
	if (!bBindingsAdded)
	{
		bBindingsAdded = true;

		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_Select", EKeys::LeftMouseButton));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_Unselect", EKeys::Escape));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_IncreaseSpeed", EKeys::Add));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_IncreaseSpeed", EKeys::MouseScrollUp));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_DecreaseSpeed", EKeys::Subtract));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_DecreaseSpeed", EKeys::MouseScrollDown));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_IncreaseFOV", EKeys::Comma));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_DecreaseFOV", EKeys::Period));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_ToggleDisplay", EKeys::BackSpace));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_FreezeRendering", EKeys::F));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_OrbitHitPoint", EKeys::O));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_OrbitCenter", EKeys::O, true));

		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_Select", EKeys::Gamepad_RightTrigger));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_IncreaseSpeed", EKeys::Gamepad_RightShoulder));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_DecreaseSpeed", EKeys::Gamepad_LeftShoulder));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_IncreaseFOV", EKeys::Gamepad_DPad_Up));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_DecreaseFOV", EKeys::Gamepad_DPad_Down));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_ToggleDisplay", EKeys::Gamepad_FaceButton_Left));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_FreezeRendering", EKeys::Gamepad_FaceButton_Top));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		if (ADebugCameraController::EnableDebugViewmodes())
		{
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_CycleViewMode", EKeys::V));
		}

		if (ADebugCameraController::EnableDebugBuffers())
		{
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_ToggleBufferVisualizationOverview", EKeys::B));
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_ToggleBufferVisualizationFull", EKeys::Enter));
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_BufferVisualizationUp", EKeys::Up));
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_BufferVisualizationDown", EKeys::Down));
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_BufferVisualizationLeft", EKeys::Left));
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("DebugCamera_BufferVisualizationRight", EKeys::Right));

			// The following axis mappings must be defined to override ADefaultPawn axis mappings when buffer visualization is enabled
			UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DebugCamera_DisableAxisMotion", EKeys::Up, 1.f));
			UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DebugCamera_DisableAxisMotion", EKeys::Down, -1.f));
			UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DebugCamera_DisableAxisMotion", EKeys::Left, -1.f));
			UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DebugCamera_DisableAxisMotion", EKeys::Right, 1.f));
		}

#endif
	}
}

void ADebugCameraController::SetupInputComponent()
{
	Super::SetupInputComponent();

	InitializeDebugCameraInputBindings();
	InputComponent->BindAction("DebugCamera_Select", IE_Pressed, this, &ADebugCameraController::SelectTargetedObject);
	InputComponent->BindAction("DebugCamera_Unselect", IE_Pressed, this, &ADebugCameraController::Unselect);

	InputComponent->BindAction("DebugCamera_IncreaseSpeed", IE_Pressed, this, &ADebugCameraController::IncreaseCameraSpeed);
	InputComponent->BindAction("DebugCamera_DecreaseSpeed", IE_Pressed, this, &ADebugCameraController::DecreaseCameraSpeed);

	InputComponent->BindAction("DebugCamera_IncreaseFOV", IE_Pressed, this, &ADebugCameraController::IncreaseFOV);
	InputComponent->BindAction("DebugCamera_IncreaseFOV", IE_Repeat, this, &ADebugCameraController::IncreaseFOV);
	InputComponent->BindAction("DebugCamera_DecreaseFOV", IE_Pressed, this, &ADebugCameraController::DecreaseFOV);
	InputComponent->BindAction("DebugCamera_DecreaseFOV", IE_Repeat, this, &ADebugCameraController::DecreaseFOV);

	InputComponent->BindAction("DebugCamera_ToggleDisplay", IE_Pressed, this, &ADebugCameraController::ToggleDisplay);
	InputComponent->BindAction("DebugCamera_FreezeRendering", IE_Pressed, this, &ADebugCameraController::ToggleFreezeRendering);
	InputComponent->BindAction("DebugCamera_OrbitHitPoint", IE_Pressed, this, &ADebugCameraController::ToggleOrbitHitPoint);
	InputComponent->BindAction("DebugCamera_OrbitCenter", IE_Pressed, this, &ADebugCameraController::ToggleOrbitCenter);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	if (ADebugCameraController::EnableDebugViewmodes())
	{
		InputComponent->BindAction("DebugCamera_CycleViewMode", IE_Pressed, this, &ADebugCameraController::CycleViewMode);
	}

	if (ADebugCameraController::EnableDebugBuffers())
	{
		InputComponent->BindAction("DebugCamera_ToggleBufferVisualizationOverview", IE_Pressed, this, &ADebugCameraController::ToggleBufferVisualizationOverviewMode);
		InputComponent->BindAction("DebugCamera_ToggleBufferVisualizationFull", IE_Pressed, this, &ADebugCameraController::ToggleBufferVisualizationFullMode);
	}

#endif

	InputComponent->BindTouch(IE_Pressed, this, &ADebugCameraController::OnTouchBegin);
	InputComponent->BindTouch(IE_Released, this, &ADebugCameraController::OnTouchEnd);
	InputComponent->BindTouch(IE_Repeat, this, &ADebugCameraController::OnFingerMove);
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

void ADebugCameraController::SetupBufferVisualizationOverviewInput()
{
	if (InputComponent)
	{
		if (bEnableBufferVisualization && !bEnableBufferVisualizationFullMode)
		{
			if (!bIsBufferVisualizationInputSetup)
			{
				InputComponent->BindAction("DebugCamera_BufferVisualizationUp", IE_Pressed, this, &ADebugCameraController::BufferVisualizationMoveUp);
				InputComponent->BindAction("DebugCamera_BufferVisualizationDown", IE_Pressed, this, &ADebugCameraController::BufferVisualizationMoveDown);
				InputComponent->BindAction("DebugCamera_BufferVisualizationRight", IE_Pressed, this, &ADebugCameraController::BufferVisualizationMoveRight);
				InputComponent->BindAction("DebugCamera_BufferVisualizationLeft", IE_Pressed, this, &ADebugCameraController::BufferVisualizationMoveLeft);
				InputComponent->BindAxis("DebugCamera_DisableAxisMotion", this, &ADebugCameraController::ConsumeAxisMotion);
				bIsBufferVisualizationInputSetup = true;
			}
		}
		else
		{
			if (bIsBufferVisualizationInputSetup)
			{
				// find any bindings that match the action names and remove them
				for (int32 CurrentBindingIndex = 0; CurrentBindingIndex < InputComponent->GetNumActionBindings(); ++CurrentBindingIndex)
				{
					const FInputActionBinding& Binding = InputComponent->GetActionBinding(CurrentBindingIndex);
					FName ActionName = Binding.GetActionName();
					if (ActionName == "DebugCamera_BufferVisualizationUp"    ||
						ActionName == "DebugCamera_BufferVisualizationDown"  ||
						ActionName == "DebugCamera_BufferVisualizationRight" ||
						ActionName == "DebugCamera_BufferVisualizationLeft")
					{
						InputComponent->RemoveActionBinding(CurrentBindingIndex);
						--CurrentBindingIndex;
					}
				}

				// find the axis binding and remove it
				for (int32 CurrentAxisBindingIndex = 0; CurrentAxisBindingIndex < InputComponent->AxisBindings.Num(); ++CurrentAxisBindingIndex)
				{
					const FInputAxisBinding& Binding = InputComponent->AxisBindings[CurrentAxisBindingIndex];
					if (Binding.AxisName == "DebugCamera_DisableAxisMotion")
					{
						InputComponent->AxisBindings.RemoveAt(CurrentAxisBindingIndex, EAllowShrinking::No);
						--CurrentAxisBindingIndex;
					}
				}
				bIsBufferVisualizationInputSetup = false;
			}
		}
	}
}

#endif 

void ADebugCameraController::OnTouchBegin(ETouchIndex::Type FingerIndex, FVector Location)
{
	if (FingerIndex == ETouchIndex::Touch1)
	{
		LastTouchDragLocation = FVector2D(Location);
	}
}

void ADebugCameraController::OnTouchEnd(ETouchIndex::Type FingerIndex, FVector Location)
{
	if (FingerIndex == ETouchIndex::Touch1)
	{
		LastTouchDragLocation = FVector2D::ZeroVector;
	}
}

static const float TouchDragRotationScale = 0.1f;

void ADebugCameraController::OnFingerMove(ETouchIndex::Type FingerIndex, FVector Location)
{
	if ( (FingerIndex == ETouchIndex::Touch1) && (!LastTouchDragLocation.IsZero()) )
	{
		FVector2D const DragDelta = (FVector2D(Location) - LastTouchDragLocation) * TouchDragRotationScale;

		AddYawInput(DragDelta.X);
		AddPitchInput(DragDelta.Y);

		LastTouchDragLocation = FVector2D(Location);
	}
}

AActor* ADebugCameraController::GetSelectedActor() const
{
	return SelectedActor.Get();
}

void ADebugCameraController::Select( FHitResult const& Hit )
{
	AActor* HitActor = Hit.HitObjectHandle.FetchActor();

	// store selection
	SelectedActor = HitActor;
	SelectedComponent = Hit.Component;
	SelectedHitPoint = Hit;

	//BP Event
	ReceiveOnActorSelected(HitActor, Hit.ImpactPoint, Hit.ImpactNormal, Hit);
}


void ADebugCameraController::Unselect()
{	
	SelectedActor.Reset();
	SelectedComponent.Reset();
}

FString ADebugCameraController::ConsoleCommand(const FString& Cmd,bool bWriteToLog)
{
	/**
	 * This is the same as PlayerController::ConsoleCommand(), except with some extra code to 
	 * give our regular PC a crack at handling the command.
	 */
	if (Player != nullptr)
	{
		UConsole* ViewportConsole = (GEngine->GameViewport != nullptr) ? GEngine->GameViewport->ViewportConsole : nullptr;
		FConsoleOutputDevice StrOut(ViewportConsole);
	
		const int32 CmdLen = Cmd.Len();
		TCHAR* CommandBuffer = (TCHAR*)FMemory::Malloc((CmdLen+1)*sizeof(TCHAR));
		TCHAR* Line = (TCHAR*)FMemory::Malloc((CmdLen+1)*sizeof(TCHAR));

		const TCHAR* Command = CommandBuffer;
		// copy the command into a modifiable buffer
		FCString::Strncpy(CommandBuffer, *Cmd.Left(CmdLen), (CmdLen+1));

		// iterate over the line, breaking up on |'s
		while (FParse::Line(&Command, Line, CmdLen+1))	// The FParse::Line function expects the full array size, including the NULL character.
		{
			if (Player->Exec( GetWorld(), Line, StrOut) == false)
			{
				Player->PlayerController = OriginalControllerRef;
				Player->Exec( GetWorld(), Line, StrOut);

				// The Player pointer could be null if the calling console command has changed the player controller back to the
				// original one. In that case, the Player pointer will be null so we can't switch the player controller back
				// (and wouldn't want to)
				if (Player != nullptr)
				{
					Player->PlayerController = this;
				}
			}
		}

		// Free temp arrays
		FMemory::Free(CommandBuffer);
		CommandBuffer = nullptr;

		FMemory::Free(Line);
		Line = nullptr;

		if (!bWriteToLog)
		{
			return *StrOut;
		}
	}

	return TEXT("");
}

void ADebugCameraController::UpdateHiddenComponents(const FVector& ViewLocation,TSet<FPrimitiveComponentId>& HiddenComponentsOut)
{
	if (OriginalControllerRef != nullptr)
	{
		OriginalControllerRef->UpdateHiddenComponents(ViewLocation, HiddenComponentsOut);
	}
}

ASpectatorPawn* ADebugCameraController::SpawnSpectatorPawn()
{
	ASpectatorPawn* SpawnedSpectator = nullptr;

	// Only spawned for the local player
	if (GetSpectatorPawn() == nullptr && IsLocalController())
	{
		AGameStateBase const* const GameState = GetWorld()->GetGameState();
		if (GameState)
		{
			FActorSpawnParameters SpawnParams;
			SpawnParams.Owner = this;
			SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			SpawnParams.ObjectFlags |= RF_Transient;	// We never want to save spectator pawns into a map

			SpawnedSpectator = GetWorld()->SpawnActor<ASpectatorPawn>((*GameState->SpectatorClass ? *GameState->SpectatorClass : ASpectatorPawn::StaticClass()), GetSpawnLocation(), GetControlRotation(), SpawnParams);
			if (SpawnedSpectator)
			{
				SpawnedSpectator->PossessedBy(this);
				SpawnedSpectator->DispatchRestart(true);
				if (SpawnedSpectator->PrimaryActorTick.bStartWithTickEnabled)
				{
					SpawnedSpectator->SetActorTickEnabled(true);
				}

				UE_LOGF(LogPlayerController, Verbose, "Spawned spectator %ls [server:%d]", *GetNameSafe(SpawnedSpectator), GetNetMode() < NM_Client);
			}
			else
			{
				UE_LOGF(LogPlayerController, Warning, "Failed to spawn spectator with class %ls", GameState->SpectatorClass ? *GameState->SpectatorClass->GetName() : TEXT("NULL"));
			}
		}
		else
		{
			// This normally happens on clients if the Player is replicated but the GameState has not yet.
			UE_LOGF(LogPlayerController, Verbose, "NULL GameState when trying to spawn spectator!");
		}
	}

	return SpawnedSpectator != nullptr ? SpawnedSpectator : Super::SpawnSpectatorPawn();
}

void ADebugCameraController::SetSpectatorPawn(ASpectatorPawn* NewSpectatorPawn)
{
	Super::SetSpectatorPawn(NewSpectatorPawn);
	if (GetSpectatorPawn())
	{
		GetSpectatorPawn()->SetActorEnableCollision(false);
		GetSpectatorPawn()->PrimaryActorTick.bTickEvenWhenPaused = bShouldPerformFullTickWhenPaused;
		USpectatorPawnMovement* SpectatorMovement = Cast<USpectatorPawnMovement>(GetSpectatorPawn()->GetMovementComponent());
		if (SpectatorMovement)
		{
			SpectatorMovement->bIgnoreTimeDilation = true;
			SpectatorMovement->PrimaryComponentTick.bTickEvenWhenPaused = bShouldPerformFullTickWhenPaused;
			InitialMaxSpeed = SpectatorMovement->MaxSpeed;
			InitialAccel = SpectatorMovement->Acceleration;
			InitialDecel = SpectatorMovement->Deceleration;
			ApplySpeedScale();
		}
	}
}

void ADebugCameraController::EndSpectatingState()
{
	DestroySpectatorPawn();
}

void ADebugCameraController::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	// if hud is existing, delete it and create new hud for debug camera
	if ( MyHUD != nullptr )
	{
		MyHUD->Destroy();
	}
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Owner = this;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want these to save into a map
	MyHUD = GetWorld()->SpawnActor<ADebugCameraHUD>( SpawnInfo );

	ChangeState(NAME_Inactive);
}

void ADebugCameraController::OnActivate( APlayerController* OriginalPC )
{
	// keep these around
	OriginalPlayer = OriginalPC->Player;
	OriginalControllerRef = OriginalPC;
	
	FVector OrigCamLoc;
	FRotator OrigCamRot;
	OriginalPC->GetPlayerViewPoint(OrigCamLoc, OrigCamRot);
	float const OrigCamFOV = OriginalPC->PlayerCameraManager->GetFOVAngle();

	ChangeState(NAME_Spectating);

	// start debug camera at original camera pos
	SetInitialLocationAndRotation(OrigCamLoc, OrigCamRot);

	if (PlayerCameraManager)
	{
		PlayerCameraManager->SetFOV( OrigCamFOV );
		PlayerCameraManager->UpdateCamera(0.0f);
	}

	// draw frustum of original camera (where you detached)
	if (DrawFrustum == nullptr)
	{
		DrawFrustum = NewObject<UDrawFrustumComponent>(OriginalPC->PlayerCameraManager);
	}
	if (DrawFrustum)
	{
		DrawFrustum->SetVisibility(true);
		OriginalPC->SetActorHiddenInGame(false);
		OriginalPC->PlayerCameraManager->SetActorHiddenInGame(false);

		DrawFrustum->FrustumAngle = OrigCamFOV;
		DrawFrustum->SetAbsolute(true, true, false);
		DrawFrustum->SetRelativeLocation(OrigCamLoc);
		DrawFrustum->SetRelativeRotation(OrigCamRot);
		DrawFrustum->RegisterComponent();

		ConsoleCommand(TEXT("show camfrustums")); //called to render camera frustums from original player camera
	}

	GetWorld()->AddController(this);
	
	//BP Event
	ReceiveOnActivate(OriginalPC);
}


void ADebugCameraController::AddCheats(bool bForce)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	Super::AddCheats(true);
#else
	Super::AddCheats(bForce);
#endif
}

void ADebugCameraController::OnDeactivate( APlayerController* RestoredPC )
{
	// restore FreezeRendering command state
	if (bIsFrozenRendering) 
	{
		ConsoleCommand(TEXT("FreezeRendering"));
		bIsFrozenRendering = false;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	if (bEnableBufferVisualization)
	{
		ToggleBufferVisualizationOverviewMode();
	}

#endif

	bIsOrbitingSelectedActor = false;

	DrawFrustum->SetVisibility(false);
	ConsoleCommand(TEXT("show camfrustums"));
	DrawFrustum->UnregisterComponent();
	RestoredPC->SetActorHiddenInGame(true);
	
	if (RestoredPC->PlayerCameraManager)
	{
		RestoredPC->PlayerCameraManager->SetActorHiddenInGame(true);
	}

	OriginalControllerRef = nullptr;
	OriginalPlayer = nullptr;

	ChangeState(NAME_Inactive);
	GetWorld()->RemoveController(this);
	
	//BP Event
	ReceiveOnDeactivate(RestoredPC);
}

void ADebugCameraController::ToggleFreezeRendering()
{
	ConsoleCommand(TEXT("FreezeRendering"));
	bIsFrozenRendering = !bIsFrozenRendering;
}

void ADebugCameraController::PreProcessInput(const float DeltaTime, const bool bGamePaused)
{
	if (bIsOrbitingSelectedActor)
	{
		PreProcessInputForOrbit(DeltaTime, bGamePaused);
	}
	else
	{
		Super::PreProcessInput(DeltaTime, bGamePaused);
	}
}

void ADebugCameraController::UpdateRotation(float DeltaTime)
{
	if (bIsOrbitingSelectedActor)
	{
		UpdateRotationForOrbit(DeltaTime);
	}
	else
	{
		Super::UpdateRotation(DeltaTime);
	}
}

void ADebugCameraController::PreProcessInputForOrbit(const float DeltaTime, const bool bGamePaused)
{
	if (bIsOrbitingSelectedActor)
	{
		if (APawn* const CurrentPawn = GetPawnOrSpectator())
		{
			if (UPawnMovementComponent* MovementComponent = CurrentPawn->GetMovementComponent())
			{
				// Reset velocity before processing input when orbiting to limit acceleration which 
				// can cause overshooting and jittering as orbit attempts to maintain a fixed radius.
				MovementComponent->Velocity = FVector::ZeroVector;
				MovementComponent->UpdateComponentVelocity();
			}
		}
	}
}

void ADebugCameraController::UpdateRotationForOrbit(float DeltaTime)
{
	APawn* const CurrentPawn = GetPawnOrSpectator();

	if (bIsOrbitingSelectedActor && CurrentPawn)
	{
		bool bUpdatePawn = false;
		FRotator ViewRotation = GetControlRotation();
	
		if (!CurrentPawn->GetLastMovementInputVector().IsZero())
		{
			FRotationMatrix ObjectToWorld(ViewRotation);
			FVector MoveDelta(CurrentPawn->GetActorLocation() - LastOrbitPawnLocation);
			FVector MoveDeltaObj = ObjectToWorld.GetTransposed().TransformVector(MoveDelta);

			// Handle either forward or lateral motion but not both, because small forward
			// motion deltas while moving laterally cause the distance from pivot to drift
			if (FMath::IsNearlyZero(MoveDeltaObj.Y, FVector::FReal(0.01)))
			{
				// Clamp delta to avoid flipping to opposite view
				const float ForwardScale = 3.0f;
				OrbitRadius = (MoveDeltaObj.X * ForwardScale > OrbitRadius - MIN_ORBIT_RADIUS) ? MIN_ORBIT_RADIUS : OrbitRadius - MoveDeltaObj.X * ForwardScale;
			}
			else
			{
				// Apply lateral movement component, constraining distance from orbit pivot
				const float LateralScale = 2.0f;
				FVector LateralDelta = ObjectToWorld.TransformVector(FVector(0.0f, MoveDeltaObj.Y * LateralScale, 0.0f));
				ViewRotation = (OrbitPivot - LastOrbitPawnLocation - LateralDelta).ToOrientationRotator();
			}
			bUpdatePawn = true;
		}
		else if (!RotationInput.IsZero())
		{
			ViewRotation += RotationInput;

			FVector Axis;
			float Angle;
			FVector OppositeViewVector = -1.0 * ViewRotation.Vector();
			FQuat::FindBetween(FVector::UpVector, OppositeViewVector).ToAxisAndAngle(Axis, Angle);

			// Clamp rotation to 10 degrees from Up vector
			const float MinAngle = UE_PI / 18.f;
			const float MaxAngle = UE_PI - MinAngle;
			if (Angle < MinAngle || Angle > MaxAngle)
			{
				float AdjustedAngle = FMath::Clamp(Angle, MinAngle, MaxAngle);
				OppositeViewVector = FQuat(Axis, AdjustedAngle).RotateVector(FVector::UpVector);
				ViewRotation = (-1.0 * OppositeViewVector).ToOrientationRotator();
			}

			bUpdatePawn = true;
		}

		if (bUpdatePawn)
		{
			LastOrbitPawnLocation = OrbitPivot - ViewRotation.Vector() * OrbitRadius;
			CurrentPawn->SetActorLocation(LastOrbitPawnLocation);

			SetControlRotation(ViewRotation);
			CurrentPawn->FaceRotation(ViewRotation);
		}
	}
}

bool ADebugCameraController::GetPivotForOrbit(FVector& PivotLocation) const
{
	if (SelectedActor.IsValid())
	{
		if (bOrbitPivotUseCenter)
		{
			FBox BoundingBox(ForceInit);
			int32 NumValidComponents = 0;

			// Use the center of the bounding box of the current selected actor as the pivot point for orbiting the camera
			int32 NumSelectedActors = 0;

			TInlineComponentArray<UMeshComponent*> MeshComponents(SelectedActor.Get());

			for (int32 ComponentIndex = 0; ComponentIndex < MeshComponents.Num(); ++ComponentIndex)
			{
				UMeshComponent* MeshComponent = MeshComponents[ComponentIndex];

				if (MeshComponent->IsRegistered() && MeshComponent->IsVisible())
				{
					BoundingBox += MeshComponent->Bounds.GetBox();
					++NumValidComponents;
				}
			}

			if (NumValidComponents > 0)
			{
				PivotLocation = BoundingBox.GetCenter();
				return true;
			}
		}
		else
		{
			PivotLocation = SelectedHitPoint.Location;
			return true;
		}
	}

	return false;
}

void ADebugCameraController::ToggleOrbit(bool bOrbitCenter)
{
	if (bIsOrbitingSelectedActor)
	{
		bIsOrbitingSelectedActor = false;
	}
	else
	{
		APawn* const CurrentPawn = GetPawnOrSpectator();
		bOrbitPivotUseCenter = bOrbitCenter;
		bIsOrbitingSelectedActor = (CurrentPawn && GetPivotForOrbit(OrbitPivot));

		if (bIsOrbitingSelectedActor)
		{
			LastOrbitPawnLocation = CurrentPawn->GetActorLocation();
			FVector ViewVector = OrbitPivot - LastOrbitPawnLocation;
			float ViewLength = ViewVector.Size();

			if (ViewLength == 0.0f)
			{
				bIsOrbitingSelectedActor = false;
				return;
			}
			else if (ViewLength >= MIN_ORBIT_RADIUS)
			{
				OrbitRadius = ViewLength;
			}
			else
			{
				LastOrbitPawnLocation = OrbitPivot - ViewVector.GetSafeNormal() * MIN_ORBIT_RADIUS;
				CurrentPawn->SetActorLocation(LastOrbitPawnLocation);
				OrbitRadius = MIN_ORBIT_RADIUS;
			}

			FRotator ViewRotation = ViewVector.ToOrientationRotator();
			SetControlRotation(ViewRotation);
			CurrentPawn->FaceRotation(ViewRotation);
		}
	}
}

void ADebugCameraController::ToggleOrbitCenter()
{
	ToggleOrbit(true);
}

void ADebugCameraController::ToggleOrbitHitPoint()
{
	ToggleOrbit(false);
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

bool ADebugCameraController::EnableDebugViewmodes()
{
	return AllowDebugViewmodes();
}

bool ADebugCameraController::EnableDebugBuffers()
{
	return AllowDebugViewmodes();
}

void ADebugCameraController::CycleViewMode()
{
	if (bEnableBufferVisualization)
	{
		ToggleBufferVisualizationOverviewMode();
	}

	if (UGameViewportClient* GameViewportClient = GetWorld()->GetGameViewport())
	{
		TArray<EViewModeIndex> DebugViewModes = UDebugCameraControllerSettings::Get()->GetCycleViewModes();

		if (DebugViewModes.Num() == 0)
		{
			UE_LOGF(LogPlayerController, Warning, "Debug camera controller settings must specify at least one view mode for view mode cycling.");
			return;
		}

		int32 CurrViewModeIndex = GameViewportClient->ViewModeIndex;
		int32 CurrIndex = LastViewModeSettingsIndex;

		int32 NextIndex = CurrIndex < DebugViewModes.Num() ? (CurrIndex + 1) % DebugViewModes.Num() : 0;
		int32 NextViewModeIndex = DebugViewModes[NextIndex];

		if (NextViewModeIndex != CurrViewModeIndex)
		{
			FString NextViewModeName = GetViewModeName((EViewModeIndex)NextViewModeIndex);

			if (!NextViewModeName.IsEmpty())
			{
				FString Cmd("VIEWMODE ");
				Cmd += NextViewModeName;
				GameViewportClient->ConsoleCommand(Cmd);
			}
			else
			{
				UE_LOGF(LogPlayerController, Warning, "Invalid view mode index %d.", NextViewModeIndex);
			}
		}

		LastViewModeSettingsIndex = NextIndex;
	}
}

TArray<FString> ADebugCameraController::GetBufferVisualizationOverviewTargets()
{
	TArray<FString> SelectedBuffers;

	// Get the list of requested buffers from the console
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.BufferVisualizationOverviewTargets"));
	if (CVar)
	{
		FString SelectedBufferNames = CVar->GetString();

		// Extract each material name from the comma separated string
		while (SelectedBufferNames.Len() && SelectedBuffers.Num() < 16)
		{
			FString Left, Right;

			// Detect last entry in the list
			if (!SelectedBufferNames.Split(TEXT(","), &Left, &Right))
			{
				Left = SelectedBufferNames;
				Right = FString();
			}

			Left.TrimStartInline();
			if (GetBufferMaterialName(*Left).IsEmpty())
			{
				SelectedBuffers.Add(TEXT(""));
			}
			else
			{
				SelectedBuffers.Add(*Left);
			}
			SelectedBufferNames = Right;
		}
	}
	else
	{
		UE_LOGF(LogPlayerController, Warning, "Console variable r.BufferVisualizationOverviewTargets is not found.");
	}
	return SelectedBuffers;
}

void ADebugCameraController::ToggleBufferVisualizationOverviewMode()
{
	SetBufferVisualizationFullMode(false);

	if (UGameViewportClient* GameViewportClient = GetWorld()->GetGameViewport())
	{
		bEnableBufferVisualization = !bEnableBufferVisualization;

		FString Cmd("VIEWMODE ");

		if (bEnableBufferVisualization)
		{
			Cmd += GetViewModeName(VMI_VisualizeBuffer);

			TArray<FString> SelectedBuffers = GetBufferVisualizationOverviewTargets();

			if (CurrSelectedBuffer.IsEmpty() || !SelectedBuffers.Contains(CurrSelectedBuffer))
			{
				GetNextBuffer(SelectedBuffers, 1);
			}

			bLastDisplayEnabled = IsDisplayEnabled();
			SetDisplay(false);
		}
		else
		{
			Cmd += GetViewModeName(EViewModeIndex::VMI_Lit);
			SetDisplay(bLastDisplayEnabled);
		}

		GameViewportClient->ConsoleCommand(Cmd);

		SetupBufferVisualizationOverviewInput();
	}
}

void ADebugCameraController::UpdateVisualizeBufferPostProcessing(FFinalPostProcessSettings& InOutPostProcessingSettings)
{
	if (bEnableBufferVisualization)
	{
		FString BufferMaterialName = GetSelectedBufferMaterialName();
		if (!BufferMaterialName.IsEmpty())
		{
			InOutPostProcessingSettings.bBufferVisualizationOverviewTargetIsSelected = true;
			InOutPostProcessingSettings.BufferVisualizationOverviewSelectedTargetMaterialName = BufferMaterialName;
			return;
		}
	}

	InOutPostProcessingSettings.bBufferVisualizationOverviewTargetIsSelected = false;
	InOutPostProcessingSettings.BufferVisualizationOverviewSelectedTargetMaterialName.Empty();
}

void ADebugCameraController::GetNextBuffer(int32 Step)
{
	if (bEnableBufferVisualization && !bEnableBufferVisualizationFullMode)
	{
		TArray<FString> OverviewBuffers = GetBufferVisualizationOverviewTargets();
		GetNextBuffer(OverviewBuffers, Step);
	}
}

void ADebugCameraController::GetNextBuffer(const TArray<FString>& OverviewBuffers, int32 Step)
{
	if (bEnableBufferVisualization && !bEnableBufferVisualizationFullMode)
	{
		int32 BufferIndex = 0;

		if (!CurrSelectedBuffer.IsEmpty())
		{
			bool bFoundIndex = false;

			for (int32 Index = 0; Index < OverviewBuffers.Num(); Index++)
			{
				if (OverviewBuffers[Index] == CurrSelectedBuffer)
				{
					BufferIndex = Index;
					bFoundIndex = true;
					break;
				}
			}

			if (!bFoundIndex)
			{
				CurrSelectedBuffer.Empty();
			}
		}

		if (CurrSelectedBuffer.IsEmpty())
		{
			for (FString Buffer : OverviewBuffers)
			{
				if (!Buffer.IsEmpty())
				{
					CurrSelectedBuffer = Buffer;
					break;
				}
			}
		}
		else
		{
			int32 Incr = FMath::Abs(Step);
			int32 Min = Incr == 1 ? (BufferIndex / 4) * 4 : BufferIndex % 4;
			int32 Max = Min;
			for (int32 i = 0; i < 3 && Max + Incr < OverviewBuffers.Num(); i++) { Max += Incr; }

			auto Wrap = [&](int32 Index)
			{
				if (Index < Min)
				{
					Index = Max;
				}
				else if (Index > Max)
				{
					Index = Min;
				}

				return Index;
			};

			int32 NextIndex = Wrap(BufferIndex + Step);

			while (NextIndex != BufferIndex)
			{
				if (!OverviewBuffers[NextIndex].IsEmpty())
				{
					CurrSelectedBuffer = OverviewBuffers[NextIndex];
					break;
				}
				NextIndex = Wrap(NextIndex + Step);
			}
		}
	}
}

void ADebugCameraController::BufferVisualizationMoveUp()
{
	GetNextBuffer(-4);
}

void ADebugCameraController::BufferVisualizationMoveDown()
{
	GetNextBuffer(4);
}

void ADebugCameraController::BufferVisualizationMoveRight()
{
	GetNextBuffer(1);
}

void ADebugCameraController::BufferVisualizationMoveLeft()
{
	GetNextBuffer(-1);
}

FString ADebugCameraController::GetBufferMaterialName(const FString& InBufferName)
{
	if (!InBufferName.IsEmpty())
	{
		if (UMaterialInterface* Material = GetBufferVisualizationData().GetMaterial(*InBufferName))
		{
			return Material->GetName();
		}
	}

	return TEXT("");
}

FString ADebugCameraController::GetSelectedBufferMaterialName()
{
	return GetBufferMaterialName(CurrSelectedBuffer);
}

void ADebugCameraController::ConsumeAxisMotion(float Val)
{
	// Just ignore the axis motion.
}

void ADebugCameraController::ToggleBufferVisualizationFullMode()
{
	SetBufferVisualizationFullMode(!bEnableBufferVisualizationFullMode);
}

void ADebugCameraController::SetBufferVisualizationFullMode(bool bFullMode)
{
	if (bEnableBufferVisualizationFullMode != bFullMode)
	{
		static IConsoleVariable* ICVar = IConsoleManager::Get().FindConsoleVariable(FBufferVisualizationData::GetVisualizationTargetConsoleCommandName());
		if (ICVar)
		{
			bEnableBufferVisualizationFullMode = bFullMode;

			static const FName EmptyName = NAME_None;
			ICVar->Set(bFullMode ? *CurrSelectedBuffer : *EmptyName.ToString(), ECVF_SetByCode);

			SetupBufferVisualizationOverviewInput();
			SetDisplay(bEnableBufferVisualizationFullMode);
		}
		else
		{
			UE_LOGF(LogPlayerController, Verbose, "Console variable %ls does not exist.", FBufferVisualizationData::GetVisualizationTargetConsoleCommandName());
		}
	}
}

#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

void ADebugCameraController::SelectTargetedObject()
{
	FVector CamLoc;
	FRotator CamRot;
	GetPlayerViewPoint(CamLoc, CamRot);

	FHitResult Hit;
	FCollisionQueryParams TraceParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), true, this);
	bool const bHit = GetWorld()->LineTraceSingleByChannel(Hit, CamLoc, CamRot.Vector() * 5000.f * 20.f + CamLoc, ECC_Pawn, TraceParams);
	if( bHit)
	{
		Select(Hit);
	}
}

void ADebugCameraController::ShowDebugSelectedInfo()
{
	bShowSelectedInfo = !bShowSelectedInfo;
}

void ADebugCameraController::IncreaseCameraSpeed()
{
	SpeedScale += SPEED_SCALE_ADJUSTMENT;
	ApplySpeedScale();
}

void ADebugCameraController::DecreaseCameraSpeed()
{
	SpeedScale -= SPEED_SCALE_ADJUSTMENT;
	SpeedScale = FMath::Max(SPEED_SCALE_ADJUSTMENT, SpeedScale);
	ApplySpeedScale();
}

void ADebugCameraController::ApplySpeedScale()
{
	ASpectatorPawn* Spectator = GetSpectatorPawn();
	if (Spectator)
	{
		USpectatorPawnMovement* SpectatorMovement = Cast<USpectatorPawnMovement>(Spectator->GetMovementComponent());
		if (SpectatorMovement)
		{
			SpectatorMovement->MaxSpeed = InitialMaxSpeed * SpeedScale;
			SpectatorMovement->Acceleration = InitialAccel * SpeedScale;
			SpectatorMovement->Deceleration = InitialDecel * SpeedScale;
		}
	}
}
void ADebugCameraController::SetPawnMovementSpeedScale(const float NewSpeedScale)
{ 
	SpeedScale = NewSpeedScale;
	ApplySpeedScale();
}

void ADebugCameraController::IncreaseFOV()
{
	if (PlayerCameraManager)
	{
		PlayerCameraManager->SetFOV( PlayerCameraManager->GetFOVAngle() + 1.f );
	}
}
void ADebugCameraController::DecreaseFOV()
{
	if (PlayerCameraManager)
	{
		PlayerCameraManager->SetFOV( PlayerCameraManager->GetFOVAngle() - 1.f );
	}
}

void ADebugCameraController::ToggleDisplay()
{
	if (MyHUD)
	{
		MyHUD->ShowHUD();
	}
}

bool ADebugCameraController::IsDisplayEnabled()
{
	return (MyHUD && MyHUD->bShowHUD);
}

void ADebugCameraController::SetDisplay(bool bEnabled)
{
	if (IsDisplayEnabled() != bEnabled)
	{
		ToggleDisplay();
	}
}

