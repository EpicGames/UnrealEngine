// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DebugCameraControllerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugCameraControllerSettings)

UDebugCameraControllerSettings::UDebugCameraControllerSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SectionName = TEXT("Debug Camera Controller");

	CycleViewModes.Add(FDebugCameraControllerSettingsViewModeIndex(VMI_Lit));
	CycleViewModes.Add(FDebugCameraControllerSettingsViewModeIndex(VMI_Unlit));
	CycleViewModes.Add(FDebugCameraControllerSettingsViewModeIndex(VMI_Wireframe));
	CycleViewModes.Add(FDebugCameraControllerSettingsViewModeIndex(VMI_Lit_DetailLighting));
	CycleViewModes.Add(FDebugCameraControllerSettingsViewModeIndex(VMI_ReflectionOverride));
	CycleViewModes.Add(FDebugCameraControllerSettingsViewModeIndex(VMI_CollisionPawn));
	CycleViewModes.Add(FDebugCameraControllerSettingsViewModeIndex(VMI_CollisionVisibility));
}

TArray<EViewModeIndex> UDebugCameraControllerSettings::GetCycleViewModes()
{
	TArray<EViewModeIndex> ViewModes;

	for (FDebugCameraControllerSettingsViewModeIndex Mode : CycleViewModes)
	{
		ViewModes.Add(Mode.ViewModeIndex);
	}

	return ViewModes;
}

#if WITH_EDITOR

void UDebugCameraControllerSettings::PostLoad()
{
	Super::PostLoad();
	RemoveInvalidViewModes();
}

void UDebugCameraControllerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	RemoveInvalidViewModes();
}

void UDebugCameraControllerSettings::RemoveInvalidViewModes()
{
	UEnum* ViewModeIndexEnum = StaticEnum<EViewModeIndex>();

	for (int32 i = 0; i < CycleViewModes.Num(); i++)
	{
		FString ViewModeName = ViewModeIndexEnum->GetNameStringByIndex((int32)CycleViewModes[i].ViewModeIndex);
		if (ViewModeName.IsEmpty())
		{
			CycleViewModes.RemoveAt(i);
			i--;
		}
	}
}

#endif

