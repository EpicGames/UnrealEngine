// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
   DebugCameraInput.cpp: Native implementation for the debug camera
=============================================================================*/

#include "Engine/DebugCameraHUD.h"
#include "Engine/GameViewportClient.h"
#include "Components/MeshComponent.h"
#include "Engine/Engine.h"
#include "DrawDebugHelpers.h"
#include "Engine/World.h"
#include "GameFramework/PawnMovementComponent.h"
#include "GameFramework/SpectatorPawn.h"
#include "Engine/Canvas.h"
#include "Engine/DebugCameraController.h"
#include "Materials/MaterialInterface.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugCameraHUD)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<int32> CVarDebugCameraTraceComplex(
	TEXT("g.DebugCameraTraceComplex"),
	1,
	TEXT("Whether DebugCamera should use complex or simple collision for the line trace.\n")
	TEXT("1: complex collision, 0: simple collision"),
	ECVF_Cheat);
#endif
// ------------------
// Externals
// ------------------

ADebugCameraHUD::ADebugCameraHUD(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetHidden(false);
}

bool ADebugCameraHUD::DisplayMaterials( float X, float& Y, float DY, UMeshComponent* MeshComp )
{
	bool bDisplayedMaterial = false;
	if ( MeshComp != NULL )
	{
		FFontRenderInfo FontRenderInfo = Canvas->CreateFontRenderInfo(false, true);

		UFont* Font = GEngine->GetSmallFont();
		for ( int32 MaterialIndex = 0; MaterialIndex < MeshComp->GetNumMaterials(); ++MaterialIndex )
		{
			UMaterialInterface* Material = MeshComp->GetMaterial(MaterialIndex);
			if ( Material != NULL )
			{
				Y += DY;
				Canvas->DrawText(Font, FString::Printf(TEXT("Material: '%s'"), *Material->GetFName().ToString()), X + DY, Y, 1.f, 1.f, FontRenderInfo );
				bDisplayedMaterial = true;
			}
		}
	}
	return bDisplayedMaterial;
}

void ADebugCameraHUD::PostRender()
{
	Super::PostRender();

#if ENABLE_DRAW_DEBUG
	if (bShowHUD)
	{
		ADebugCameraController* DCC = Cast<ADebugCameraController>( PlayerOwner );
		UFont* RenderFont = GEngine->GetSmallFont();
		if( DCC != NULL )
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (ADebugCameraController::EnableDebugBuffers() && DCC->bEnableBufferVisualizationFullMode)
			{
				RenderFont = GEngine->GetMediumFont();
				FFontRenderInfo FontRenderInfo = Canvas->CreateFontRenderInfo(false, true);

				Canvas->SetDrawColor(255, 255, 64, 255);
				FString BufferText = DCC->GetSelectedBufferMaterialName();
				float xl, yl;
				Canvas->StrLen(RenderFont, BufferText, xl, yl);
				float X = Canvas->SizeX * 0.05f;
				float Y = Canvas->SizeY < 3.0f * yl ? 0.0f : Canvas->SizeY - 3.0f * yl;
				Canvas->DrawText(RenderFont, BufferText, X, Y, 1.f, 1.f, FontRenderInfo);

				return;
			}
#endif	

			FFontRenderInfo FontRenderInfo = Canvas->CreateFontRenderInfo(false, true);

			Canvas->SetDrawColor(64, 64, 255, 255);
			FString MyText = "Debug Camera";
			float xl, yl;
			Canvas->StrLen(RenderFont, MyText, xl, yl);
			float X = FMath::FloorToFloat(Canvas->SizeX * 0.05f);
			float Y = yl;//*1.67;
			yl += 2*Y;
			Canvas->DrawText(RenderFont, MyText, X, yl, 1.f, 1.f, FontRenderInfo);

			Canvas->SetDrawColor(200, 200, 128, 255);

			FVector const CamLoc = DCC->PlayerCameraManager->GetCameraLocation();
			FRotator const CamRot = DCC->PlayerCameraManager->GetCameraRotation();
			float const CamFOV = DCC->PlayerCameraManager->GetFOVAngle();

			yl += Y;
			
			FString const LocRotString = FString::Printf(TEXT("Loc=(%.1f, %.1f, %.1f) Rot=(%.1f, %.1f, %.1f)"), CamLoc.X, CamLoc.Y, CamLoc.Z, CamRot.Pitch, CamRot.Yaw, CamRot.Roll);
			Canvas->DrawText(RenderFont, LocRotString, X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;
			
			FString const FOVString = FString::Printf(TEXT("HFOV=%.1f"), CamFOV);
			Canvas->DrawText(RenderFont, FOVString, X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;
			
			FString const SpeedScaleString = FString::Printf(TEXT("SpeedScale=%.2fx"), DCC->SpeedScale);
			Canvas->DrawText(RenderFont, SpeedScaleString, X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;
			
			FString const SpeedString = FString::Printf(TEXT("MaxSpeed=%.1f"), DCC->GetSpectatorPawn() && DCC->GetSpectatorPawn()->GetMovementComponent() ? DCC->GetSpectatorPawn()->GetMovementComponent()->GetMaxSpeed() : 0.f);
			Canvas->DrawText(RenderFont, SpeedString, X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (ADebugCameraController::EnableDebugViewmodes())
			{
				FString const ViewModeString = FString::Printf(TEXT("ViewMode=%s"), GetWorld()->GetGameViewport() ? GetViewModeName((EViewModeIndex)GetWorld()->GetGameViewport()->ViewModeIndex) : TEXT("<Not found>"));
				Canvas->DrawText(RenderFont, ViewModeString, X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;
			}
#endif
			const TCHAR* CVarComplexName = TEXT("g.DebugCameraTraceComplex");
			bool bTraceComplex = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			bTraceComplex = CVarDebugCameraTraceComplex.GetValueOnGameThread() != 0;
#endif

			FCollisionQueryParams TraceParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), bTraceComplex, this);
			TraceParams.bReturnPhysicalMaterial = true;
			FHitResult Hit;
			bool bHit = GetWorld()->LineTraceSingleByChannel(Hit, CamLoc, CamRot.Vector() * 100000.f + CamLoc, ECC_Visibility, TraceParams);

			yl += Y;
			Canvas->DrawText(RenderFont, FString::Printf(TEXT("Trace info (%s = %d):"), CVarComplexName, bTraceComplex ? 1 : 0), X, yl, 1.f, 1.f, FontRenderInfo);

			if( bHit )
			{
				const FActorInstanceHandle& HitHandle = Hit.HitObjectHandle;
				yl += Y;
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("HitLoc:%s HitNorm:%s"), *Hit.Location.ToString(), *Hit.Normal.ToString() ), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("HitDist: %f"), Hit.Distance), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("HitObject: '%s'"), *HitHandle.GetName()), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("HitComponent: '%s'"), Hit.Component.Get() ? *Hit.Component.Get()->GetFName().ToString() : TEXT("<NULL>")), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;
				UClass* HitClass = HitHandle.GetRepresentedClass();
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("HitHandle Class: '%s'"), HitClass ? *HitClass->GetName() : TEXT("<Not Found>") ), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y; AActor* HitActor = HitHandle.FetchActor();
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("HitActorPath: '%s'"), HitActor ? *HitActor->GetPathName() : TEXT("<Not Found>")), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("HitPhysMat: '%s'"), Hit.PhysMaterial.Get() ? *Hit.PhysMaterial.Get()->GetPathName() : TEXT("<Not Found>")), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;

				bool bFoundMaterial = false;
				if ( Hit.Component != NULL )
				{
					bFoundMaterial = DisplayMaterials( X, yl, Y, Cast<UMeshComponent>(Hit.Component.Get()) );
				}
				else
				{
					for (UActorComponent* Component : GetComponents())
					{
						UMeshComponent* MeshComp = Cast<UMeshComponent>(Component);
						if (MeshComp && MeshComp->IsRegistered())
						{
							bFoundMaterial = bFoundMaterial || DisplayMaterials( X, yl, Y, MeshComp );	
						}
					}
				}
				if ( bFoundMaterial == false )
				{
					yl += Y;
					Canvas->DrawText(RenderFont, TEXT("Material: NULL"), X + Y, yl, 1.f, 1.f, FontRenderInfo );
				}

				if (!DCC->bIsOrbitingSelectedActor)
				{
					DrawDebugLine(GetWorld(), Hit.Location, Hit.Location + Hit.Normal*30.f, FColor::White);

					if (DCC->SelectedActor.IsValid())
					{
						DrawDebugLine(GetWorld(), DCC->SelectedHitPoint.Location, DCC->SelectedHitPoint.Location + DCC->SelectedHitPoint.Normal*30.f, FColor::Red);
					}
				}
			}
			else
			{
				yl += Y;
				Canvas->DrawText( RenderFont, TEXT("No trace Hit"), X, yl, 1.f, 1.f, FontRenderInfo);
			}

			if (DCC->bShowSelectedInfo && DCC->SelectedActor.IsValid())
			{
				if (DCC->bIsOrbitingSelectedActor)
				{
					if (DCC->bOrbitPivotUseCenter)
					{
						yl += Y;
						Canvas->DrawText(RenderFont, FString::Printf(TEXT("Orbiting actor (center): '%s'"), *DCC->SelectedActor->GetFName().ToString()), X, yl, 1.f, 1.f, FontRenderInfo);
					}
					else
					{
						yl += Y;
						Canvas->DrawText(RenderFont, FString::Printf(TEXT("Orbiting actor (hitpoint): '%s'"), *DCC->SelectedActor->GetFName().ToString()), X, yl, 1.f, 1.f, FontRenderInfo);
					}
				}
				else
				{
					yl += Y;
					Canvas->DrawText(RenderFont, FString::Printf(TEXT("Selected actor: '%s'"), *DCC->SelectedActor->GetFName().ToString()), X, yl, 1.f, 1.f, FontRenderInfo);
				}
				DisplayMaterials(X, yl, Y, Cast<UMeshComponent>(DCC->SelectedComponent));
			}


			// controls display
			yl += Y*2;
			
			Canvas->SetDrawColor(64, 64, 255, 255);
			Canvas->DrawText(RenderFont, TEXT("Controls"), X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;
			
			Canvas->SetDrawColor(200, 200, 128, 255);
			Canvas->DrawText(RenderFont, TEXT("FOV +/-: ,/. or DPad Up/Down"), X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;
			
			Canvas->DrawText(RenderFont, TEXT("Speed +/-: MouseWheel or +/- or LB/RB"), X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;
			
			Canvas->DrawText(RenderFont, TEXT("Freeze Rendering: F or YButton"), X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;			
			
			Canvas->DrawText(RenderFont, TEXT("Toggle Display: BackSpace or XButton"), X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;

			Canvas->DrawText(RenderFont, TEXT("Orbit Selected Hitpoint: O"), X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;

			Canvas->DrawText(RenderFont, TEXT("Orbit Selected Actor: Shift-O"), X, yl, 1.f, 1.f, FontRenderInfo);
			yl += Y;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

			if (ADebugCameraController::EnableDebugBuffers())
			{
				Canvas->DrawText(RenderFont, TEXT("Toggle Buffer Visualization: B ('Enter' to toggle full buffer mode)"), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;
			}

			if (ADebugCameraController::EnableDebugViewmodes())
			{
				Canvas->DrawText(RenderFont, TEXT("Cycle View Mode: V"), X, yl, 1.f, 1.f, FontRenderInfo);
				yl += Y;
			}
#endif
		}
	}
#endif
}

