// Copyright Epic Games, Inc. All Rights Reserved.

#include "DebugCameraManager.h"
#include "Engine/DebugCameraController.h"
#include "Engine/GameInstance.h"
#include "Engine/Player.h"
#include "Engine/World.h"

FDebugCameraManager::FDebugCameraManager()
{
	DebugCameraControllerClass = ADebugCameraController::StaticClass();

	GameInstanceWorldChangedHandle = FWorldDelegates::OnGameInstanceWorldChanged.AddRaw(this, &FDebugCameraManager::HandleGameInstanceWorldChanged);
}

FDebugCameraManager::~FDebugCameraManager()
{
	FWorldDelegates::OnGameInstanceWorldChanged.Remove(GameInstanceWorldChangedHandle);
}

void FDebugCameraManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (TPair<TObjectKey<UPlayer>, FDebugCameraControllerStatus>& Pair : PlayerToDebugCameraControllerMap)
	{
		Collector.AddReferencedObject(Pair.Value.DebugCameraController);
	}
}

FDebugCameraManager& FDebugCameraManager::Get()
{
	static FDebugCameraManager DebugCameraManager;
	return DebugCameraManager;
}

void FDebugCameraManager::HandleGameInstanceWorldChanged(UGameInstance* GameInstance, UWorld* OldWorld, UWorld* NewWorld)
{
	PlayerToDebugCameraControllerMap.Reset();
}

void FDebugCameraManager::EnableDebugCamera(APlayerController* PlayerController)
{
	if (PlayerController && PlayerController->Player && PlayerController->IsLocalPlayerController())
	{
		ADebugCameraController* DebugCameraController = nullptr;
		if (FDebugCameraControllerStatus* DebugCameraControllerStatus = PlayerToDebugCameraControllerMap.Find(PlayerController->Player))
		{
			if (DebugCameraControllerStatus->bIsActive)
			{
				return;
			}
			DebugCameraController = DebugCameraControllerStatus->DebugCameraController;
		}

		if (DebugCameraController == nullptr)
		{
			// spawn if necessary
			FActorSpawnParameters SpawnInfo;
			SpawnInfo.Instigator = PlayerController->GetInstigator();
			DebugCameraController = PlayerController->GetWorld()->SpawnActor<ADebugCameraController>(DebugCameraControllerClass, SpawnInfo);
			if (DebugCameraController != nullptr)
			{
				PlayerToDebugCameraControllerMap.Add(PlayerController->Player, FDebugCameraControllerStatus(DebugCameraController, false));
			}
		}

		if (ensure(DebugCameraController != nullptr))
		{
			PlayerToDebugCameraControllerMap[PlayerController->Player].bIsActive = true;

			// set up new controller
			DebugCameraController->OnActivate(PlayerController);

			// then switch to it
			PlayerController->Player->SwitchController(DebugCameraController);
		}
	}
}

void FDebugCameraManager::DisableDebugCamera(APlayerController* PlayerController)
{
	FDebugCameraControllerStatus* DebugCameraControllerStatus = GetDebugCameraControllerStatus(PlayerController);
	if (DebugCameraControllerStatus)
	{
		if (DebugCameraControllerStatus->bIsActive)
		{
			TObjectPtr<ADebugCameraController> DebugCameraController = DebugCameraControllerStatus->DebugCameraController;
			if (DebugCameraController != nullptr && DebugCameraController->OriginalPlayer != nullptr)
			{
				DebugCameraController->OriginalPlayer->SwitchController(DebugCameraController->OriginalControllerRef);
				DebugCameraController->OnDeactivate(DebugCameraController->OriginalControllerRef);
				DebugCameraController->Player = nullptr;
				DebugCameraControllerStatus->bIsActive = false;
			}
		}
	}
}

bool FDebugCameraManager::IsDebugCameraActive(const APlayerController* PlayerController) const
{
	const FDebugCameraControllerStatus* DebugCameraControllerStatus = GetDebugCameraControllerStatus(PlayerController);
	return DebugCameraControllerStatus && DebugCameraControllerStatus->bIsActive;
}

ADebugCameraController* FDebugCameraManager::GetCameraController(const APlayerController* PlayerController)
{
	if (const FDebugCameraControllerStatus* DebugCameraControllerStatus = GetDebugCameraControllerStatus(PlayerController))
	{
		return DebugCameraControllerStatus->DebugCameraController;
	}
	return nullptr;
}

FDebugCameraManager::FDebugCameraControllerStatus* FDebugCameraManager::GetDebugCameraControllerStatus(const APlayerController* PlayerController)
{
	if (PlayerController != nullptr)
	{
		if (PlayerController->Player != nullptr)
		{
			if (FDebugCameraControllerStatus* DebugCameraControllerStatus = PlayerToDebugCameraControllerMap.Find(PlayerController->Player))
			{
				return DebugCameraControllerStatus;
			}
		}
		else
		{
			for (TPair<TObjectKey<UPlayer>, FDebugCameraControllerStatus>& PlayerToDebugCameraController : PlayerToDebugCameraControllerMap)
			{
				if (PlayerToDebugCameraController.Value.DebugCameraController && PlayerToDebugCameraController.Value.DebugCameraController->OriginalControllerRef == PlayerController)
				{
					return &PlayerToDebugCameraController.Value;
				}
			}
		}
	}
	return nullptr;
}

const FDebugCameraManager::FDebugCameraControllerStatus* FDebugCameraManager::GetDebugCameraControllerStatus(const APlayerController* PlayerController) const
{
	if (PlayerController != nullptr)
	{
		if (PlayerController->Player != nullptr)
		{
			if (const FDebugCameraControllerStatus* DebugCameraControllerStatus = PlayerToDebugCameraControllerMap.Find(PlayerController->Player))
			{
				return DebugCameraControllerStatus;
			}
		}
		else
		{
			for (const TPair<TObjectKey<UPlayer>, FDebugCameraControllerStatus>& PlayerToDebugCameraController : PlayerToDebugCameraControllerMap)
			{
				if (PlayerToDebugCameraController.Value.DebugCameraController && PlayerToDebugCameraController.Value.DebugCameraController->OriginalControllerRef == PlayerController)
				{
					return &PlayerToDebugCameraController.Value;
				}
			}
		}
	}
	return nullptr;
}