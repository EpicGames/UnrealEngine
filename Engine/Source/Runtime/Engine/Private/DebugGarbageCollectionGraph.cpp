// Copyright Epic Games, Inc. All Rights Reserved.

#include "DebugGarbageCollectionGraph.h"

#include "Debug/DebugDrawService.h"
#include "Engine/GameEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugGarbageCollectionGraph)

/*
	TODO additional features for a future changeset:
	- compute and display the all-time (not windowed) average
	  collection time so far
 */

int32 UDebugGarbageCollectionGraph::SafeDurationThreshold = 4;

void UDebugGarbageCollectionGraph::StartDrawing()
{
	if (DrawHandle.IsValid())
	{
		return;
	}
	DrawHandle = UDebugDrawService::Register(TEXT("Game"),
		FDebugDrawDelegate::CreateUObject(this,
			&UDebugGarbageCollectionGraph::Draw));
}

void UDebugGarbageCollectionGraph::StopDrawing()
{
	if (!DrawHandle.IsValid())
	{
		return;
	}
	UDebugDrawService::Unregister(DrawHandle);
	DrawHandle.Reset();
}

void UDebugGarbageCollectionGraph::Draw(UCanvas* Canvas, APlayerController*)
{
	static constexpr double FrameSeconds     = 1 / 60.0;
	static constexpr int    ViewX            = 20;
	static constexpr int    ViewY            = 20;
	static constexpr int    ViewWidth        = 400;
	static constexpr int    ViewHeight       = 400;
	static constexpr int    OneMinuteSeconds = 60;
	static constexpr double WidthSeconds     = OneMinuteSeconds * 0.5;
	static constexpr double HeightSeconds    = FrameSeconds * 10;
	static constexpr double PixelsPerSecond  = ViewWidth / WidthSeconds;

	const double LastGCTime = GetLastGCTime();
	const double CurTime    = FPlatformTime::Seconds();

	if (LastGCTime != History.back().Time)
	{
		History.push_back({LastGCTime, GetLastGCDuration()});
	}
	while (!History.empty() && CurTime - History.front().Time > WidthSeconds)
	{
		History.erase(History.begin());
	}

	auto&& DrawLine =
		[Canvas](double X0, double Y0, double X1, double Y1, const FLinearColor& Color, double Thickness)
		{
			FCanvasLineItem Line{FVector2D{X0, Y0}, FVector2D{X1, Y1}};
			Line.SetColor(Color);
			Line.LineThickness = Thickness;
			Canvas->DrawItem(Line);
		};

	auto&& DrawString =
		[Canvas](const FString& Str, int X, int Y)
		{
			FCanvasTextItem Text{FVector2D(X, Y), FText::FromString(Str), GEngine->GetSmallFont(), FLinearColor::Yellow};
			Text.EnableShadow(FLinearColor::Black);
			Text.bCentreX = true;
			Text.bCentreY = true;
			Canvas->DrawItem(Text);
		};

	{ // draw graph edges + label
		const FLinearColor Color = FLinearColor::White;
		DrawLine(ViewX, ViewY + ViewHeight, ViewX + ViewWidth, ViewY + ViewHeight, Color, 1);
		DrawLine(ViewX, ViewY, ViewX, ViewY + ViewHeight, Color, 1);
		DrawLine(ViewX + ViewWidth, ViewY, ViewX + ViewWidth, ViewY + ViewHeight, Color, 1);
		DrawString(TEXT("GC Mark Durations"), ViewX + ViewWidth * 0.5, ViewY + ViewHeight + 16);
	}

	for (int I = History.size() - 1; I >= 0; --I)
	{
		const auto&        Item = History[I];
		const double       X    = ViewX + ViewWidth - PixelsPerSecond * (CurTime - Item.Time);
		const double       H    = ViewHeight * (Item.Duration / HeightSeconds);
		const FLinearColor Color =
			Item.Duration <= (SafeDurationThreshold * 0.001)
				? FLinearColor::Green
				: FLinearColor::Red;
		const int LineThickness = 3;

		DrawLine(X, ViewY + ViewHeight - 1, X, ViewY + ViewHeight - H, Color, LineThickness);
		DrawString(
			FString::Printf(TEXT("%dms"), int(Item.Duration * 1000)), X, ViewY + ViewHeight - H - 11);
	}
}

static UDebugGarbageCollectionGraph* GDebugGraph;

static void ShowDebugGraph(FOutputDevice&)
{
	if (GDebugGraph)
	{
		return;
	}
	GDebugGraph = NewObject<UDebugGarbageCollectionGraph>();
	GDebugGraph->AddToRoot();
	GDebugGraph->StartDrawing();
}

static void HideDebugGraph(FOutputDevice&)
{
	if (!GDebugGraph)
	{
		return;
	}
	GDebugGraph->StopDrawing();
	GDebugGraph->RemoveFromRoot();
	GDebugGraph = nullptr;
}

static FAutoConsoleCommandWithOutputDevice
	GShowDebugGraphCmd(
		TEXT("gc.DebugGraphShow"),
		TEXT("Show GC debug graph. "
			 "(See also: DebugGraphSafeDurationThresholdMs)"),
		FConsoleCommandWithOutputDeviceDelegate::CreateStatic(&ShowDebugGraph));

static FAutoConsoleCommandWithOutputDevice
	GHideDebugGraphCmd(
		TEXT("gc.DebugGraphHide"),
		TEXT("Hide GC debug graph."),
		FConsoleCommandWithOutputDeviceDelegate::CreateStatic(&HideDebugGraph));

static FAutoConsoleVariableRef CVarDebugGraphSafeDurationThreshold(
	TEXT("gc.DebugGraphSafeDurationThresholdMs"),
	UDebugGarbageCollectionGraph::SafeDurationThreshold,
	TEXT("GC Debug Graph: Safe GC duration threshold (in milliseconds)."),
	ECVF_Default);
