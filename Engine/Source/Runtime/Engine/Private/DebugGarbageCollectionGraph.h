// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Canvas.h"
#include <vector>

#include "DebugGarbageCollectionGraph.generated.h"

UCLASS()
class UDebugGarbageCollectionGraph : public UObject
{
	GENERATED_BODY()

public:
	void StartDrawing();
	void StopDrawing();

	static int32 SafeDurationThreshold;

private:
	void Draw(UCanvas* Canvas, class APlayerController* PC);

	struct HistoryItem
	{
		double Time;
		double Duration;
	};
	std::vector<HistoryItem> History = {
		{0, 0}
    };

	FDelegateHandle DrawHandle;
};
