// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugRenderSceneProxy.h: Useful scene proxy for rendering non performance-critical information.


=============================================================================*/

#include "DebugRenderSceneProxy.h"
#include "DynamicMeshBuilder.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "Debug/DebugDrawService.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MeshElementCollector.h"
#include "SceneInterface.h"

static TAutoConsoleVariable<bool> CVarDebugRenderAllowFrustumCulling(
	TEXT("r.DebugRender.AllowFrustumCulling"), true,
	TEXT("Allows to cull debug shapes against the view frustum. This helps in high item number situations but incurs a price on the CPU."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarDebugRenderOverrideFarClippingPlane(
	TEXT("r.DebugRender.OverrideFarClippingPlane"), 0.0f,
	TEXT("Allows to override the far clipping plane for debug shapes and text (in cm), only effective if > 0.0 and if r.DebugRender.AllowFrustumCulling is enabled"),
	ECVF_RenderThreadSafe);

// UE::DebugDrawHelper
FVector3f UE::DebugDrawHelper::GetScaleAdjustedScreenLocation(const TNotNull<const UCanvas*> Canvas, const FVector WorldLocation)
{
	return GetScaleAdjustedScreenLocation(Canvas->SceneView, *Canvas->Canvas, WorldLocation);
}
FVector3f UE::DebugDrawHelper::GetScaleAdjustedScreenLocation(const TNotNull<const FSceneView*> View, const FCanvas& Canvas, const FVector WorldLocation)
{
	const FVector3f ScreenLocation = LWC::NarrowWorldPositionChecked(View->Project(WorldLocation));
	const float InvDPIScale = 1.f / Canvas.GetDPIScale();
	const FIntRect& UnscaledViewRect = View->UnscaledViewRect;
	const float HalfWidth = UnscaledViewRect.Width() * 0.5f;
	const float HalfHeight = UnscaledViewRect.Height() * 0.5f;
	const FIntRect HalfDelta = (View->UnconstrainedViewRect - UnscaledViewRect) / 2;

	return {(HalfDelta.Width() + (1.0f + ScreenLocation.X) * HalfWidth) * InvDPIScale
		, (HalfDelta.Height() + (1.0f - ScreenLocation.Y) * HalfHeight) * InvDPIScale
		, ScreenLocation.Z};
}

// FDebugRenderSceneProxy::FMesh
FDebugRenderSceneProxy::FMesh::FMesh() = default;
FDebugRenderSceneProxy::FMesh::~FMesh() = default;

FDebugRenderSceneProxy::FMesh::FMesh(const FMesh& Other)
	: Vertices(Other.Vertices)
	, Indices(Other.Indices)
	, Box(Other.Box)
	, Color(Other.Color)
{
}

// FPrimitiveSceneProxy interface
FDebugRenderSceneProxy::FDebugRenderSceneProxy(const UPrimitiveComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
	, ViewFlagIndex(uint32(FEngineShowFlags::FindIndexByName(TEXT("Game"))))
	, TextWithoutShadowDistance(1500)
	, ViewFlagName(TEXT("Game"))
	, DrawType(WireMesh)
	, DrawAlpha(100)
{
}

FDebugRenderSceneProxy::FDebugRenderSceneProxy(const FDebugRenderSceneProxy&) = default;

FDebugRenderSceneProxy::~FDebugRenderSceneProxy() = default;

void FDebugDrawDelegateHelper::RegisterDebugDrawDelegateInternal()
{
	// note that it's possible at this point for State == RegisteredState since RegisterDebugDrawDelegateInternal can get 
	// called in multiple scenarios, most notably blueprint recompilation or changing level visibility in the editor.
	if (State == InitializedState)
	{
		DebugTextDrawingDelegate = FDebugDrawDelegate::CreateRaw(this, &FDebugDrawDelegateHelper::HandleDrawDebugLabels);
		DebugTextDrawingDelegateHandle = UDebugDrawService::Register(*ViewFlagName, DebugTextDrawingDelegate);
		State = RegisteredState;
	}
}

void FDebugDrawDelegateHelper::InitDelegateHelper(const FDebugRenderSceneProxy* InSceneProxy)
{
	check(IsInParallelGameThread() || IsInGameThread());

	Texts.Reset();
	Texts.Append(InSceneProxy->Texts);
	ViewFlagName = InSceneProxy->ViewFlagName;
	TextWithoutShadowDistance = InSceneProxy->TextWithoutShadowDistance;
	State = (State == UndefinedState) ? InitializedState : State;
	FarClippingDistance = InSceneProxy->FarClippingDistance;
	AssociatedWorld = InSceneProxy->GetScene().GetWorld();
}

void FDebugDrawDelegateHelper::RequestRegisterDebugDrawDelegate(FRegisterComponentContext* Context)
{
	bDeferredRegister = Context != nullptr;

	if (!bDeferredRegister)
	{
		RegisterDebugDrawDelegateInternal();
	}
}

void FDebugDrawDelegateHelper::ProcessDeferredRegister()
{
	if (bDeferredRegister)
	{
		RegisterDebugDrawDelegateInternal();
		bDeferredRegister = false;
	}
}

void FDebugDrawDelegateHelper::UnregisterDebugDrawDelegate()
{
	// note that it's possible at this point for State == InitializedState since UnregisterDebugDrawDelegate can get 
	// called in multiple scenarios, most notably blueprint recompilation or changing level visibility in the editor.
	if (State == RegisteredState)
	{
		check(DebugTextDrawingDelegate.IsBound());
		UDebugDrawService::Unregister(DebugTextDrawingDelegateHandle);
		State = InitializedState;
	}
}

void  FDebugDrawDelegateHelper::ReregisterDebugDrawDelegate()
{
	ensureMsgf(State != UndefinedState, TEXT("DrawDelegate is in an invalid State: %i !"), State);
	if (State == RegisteredState)
	{
		UnregisterDebugDrawDelegate();
		RegisterDebugDrawDelegateInternal();
	}
}

uint32 FDebugRenderSceneProxy::GetAllocatedSize(void) const 
{ 
	return	FPrimitiveSceneProxy::GetAllocatedSize() + 
		Cylinders.GetAllocatedSize() + 
		Circles.GetAllocatedSize() +
		ArrowLines.GetAllocatedSize() + 
		Stars.GetAllocatedSize() + 
		DashedLines.GetAllocatedSize() + 
		Lines.GetAllocatedSize() + 
		Boxes.GetAllocatedSize() + 
		Spheres.GetAllocatedSize() +
		Texts.GetAllocatedSize() +
		CoordinateSystems.GetAllocatedSize();
}


void FDebugDrawDelegateHelper::HandleDrawDebugLabels(UCanvas* Canvas, APlayerController* PlayerController)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDebugDrawDelegateHelper::OnDrawDebugLabels);

	check (Canvas);

	// Ignore if the helper's associated scene doesn't match the one associated to the canvas
	if (Canvas->Canvas != nullptr
		&& AssociatedWorld != Canvas->Canvas->GetScene()->GetWorld())
	{
		return;
	}

	DrawDebugLabels(Canvas, PlayerController);
}

void FDebugDrawDelegateHelper::DrawDebugLabels(UCanvas* Canvas, APlayerController*)
{
	const FColor OldDrawColor = Canvas->DrawColor;
	const FFontRenderInfo FontRenderInfo = Canvas->CreateFontRenderInfo(true, false);
	const FFontRenderInfo FontRenderInfoWithShadow = Canvas->CreateFontRenderInfo(true, true);

	Canvas->SetDrawColor(FColor::White);

	UFont* RenderFont = GEngine->GetSmallFont();

	const FSceneView* View = Canvas->SceneView;
	const bool bAllowFrustumCulling = CVarDebugRenderAllowFrustumCulling.GetValueOnGameThread();
	const FConvexVolume AdjustedFrustum = FDebugRenderSceneProxy::AdjustViewFrustumForFarClipping(View, FarClippingDistance); 
	for (auto It = Texts.CreateConstIterator(); It; ++It)
	{
		if (!bAllowFrustumCulling || FDebugRenderSceneProxy::PointInFrustum(It->Location, AdjustedFrustum))
		{
			const FVector3f ScreenLoc = UE::DebugDrawHelper::GetScaleAdjustedScreenLocation(Canvas, It->Location);
			const FFontRenderInfo& FontInfo = TextWithoutShadowDistance >= 0 ? (FDebugRenderSceneProxy::PointInRange(It->Location, View, TextWithoutShadowDistance) ? FontRenderInfoWithShadow : FontRenderInfo) : FontRenderInfo;
			Canvas->SetDrawColor(It->Color);
			Canvas->DrawText(RenderFont, It->Text, ScreenLoc.X, ScreenLoc.Y, 1, 1, FontInfo);
		}
	}

	Canvas->SetDrawColor(OldDrawColor);
}

SIZE_T FDebugRenderSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FDebugRenderSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const 
{
	QUICK_SCOPE_CYCLE_COUNTER( STAT_DebugRenderSceneProxy_GetDynamicMeshElements );

	FMaterialCache DefaultMaterialCache(Collector);
	FMaterialCache SolidMeshMaterialCache(Collector, /**bUseLight*/ true, SolidMeshMaterial.Get());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			GetDynamicMeshElementsForView(View, ViewIndex, ViewFamily, VisibilityMap, Collector, DefaultMaterialCache, SolidMeshMaterialCache);
		}
	}
}

void FDebugRenderSceneProxy::GetDynamicMeshElementsForView(const FSceneView* View, const int32 ViewIndex, const FSceneViewFamily& ViewFamily, const uint32 VisibilityMap, FMeshElementCollector& Collector, FMaterialCache& DefaultMaterialCache, FMaterialCache& SolidMeshMaterialCache) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDebugRenderSceneProxy::GetDynamicMeshElementsForView);

	FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

	const bool bAllowFrustumCulling = CVarDebugRenderAllowFrustumCulling.GetValueOnRenderThread();
	const FConvexVolume AdjustedFrustum = AdjustViewFrustumForFarClipping(View, FarClippingDistance);;

	// Draw Lines
	const int32 LinesNum = Lines.Num();
	PDI->AddReserveLines(SDPG_World, LinesNum, false, false);
	for (const FDebugLine& Line : Lines)
	{
		if (!bAllowFrustumCulling || SegmentInFrustum(Line.Start, Line.End, AdjustedFrustum))
		{
			Line.Draw(PDI);
		}
	}

	// Draw Dashed Lines
	for (const FDashedLine& Dash : DashedLines)
	{
		if (!bAllowFrustumCulling || SegmentInFrustum(Dash.Start, Dash.End, AdjustedFrustum))
		{
			Dash.Draw(PDI);
		}
	}

	// Draw Circles
	for (const FCircle& Circle : Circles)
	{
		// The idea is to have cheap test, not a precise one : a bounding sphere will do : 
		if (!bAllowFrustumCulling || SphereInFrustum(Circle.Center, Circle.Radius, AdjustedFrustum))
		{
			Circle.Draw(PDI, (Circle.DrawTypeOverride != EDrawType::Invalid) ? Circle.DrawTypeOverride : DrawType, DrawAlpha, DefaultMaterialCache, ViewIndex, Collector);
		}
	}

	// Draw Arrows
	const uint32 ArrowsNum = ArrowLines.Num();
	PDI->AddReserveLines(SDPG_World, 5 * ArrowsNum, false, false);
	for (const FArrowLine& ArrowLine : ArrowLines)
	{
		if (!bAllowFrustumCulling || SegmentInFrustum(ArrowLine.Start, ArrowLine.End, AdjustedFrustum))
		{
			ArrowLine.Draw(PDI);
		}
	}

	// Draw Stars
	for (const FWireStar& Star : Stars)
	{
		if (!bAllowFrustumCulling || SphereInFrustum(Star.Position, Star.Size, AdjustedFrustum))
		{
			Star.Draw(PDI);
		}
	}

	// Draw Cylinders
	for (const FWireCylinder& Cylinder : Cylinders)
	{
		// The idea is to have cheap test, not a precise one : a bounding sphere will do : 
		double BoundingSphereRadius = FMath::Sqrt(FMath::Square(Cylinder.HalfHeight) + 2.0 * FMath::Square(Cylinder.Radius));
		if (!bAllowFrustumCulling || SphereInFrustum(Cylinder.Base, BoundingSphereRadius, AdjustedFrustum))
		{
			Cylinder.Draw(PDI, (Cylinder.DrawTypeOverride != EDrawType::Invalid) ? Cylinder.DrawTypeOverride : DrawType, DrawAlpha, DefaultMaterialCache, ViewIndex, Collector);
		}
	}

	// Draw Boxes
	for (const FDebugBox& Box : Boxes)
	{
		FVector Center, Extents;
		FBox WorldBox = Box.Box.TransformBy(Box.Transform);
		WorldBox.GetCenterAndExtents(Center, Extents);
		if (!bAllowFrustumCulling || BoxInFrustum(Center, Extents, AdjustedFrustum))
		{
			Box.Draw(PDI, (Box.DrawTypeOverride != EDrawType::Invalid) ? Box.DrawTypeOverride : DrawType, DrawAlpha, DefaultMaterialCache, ViewIndex, Collector);
		}
	}

	// Draw Cones
	for (const FCone& Cone : Cones)
	{
		// The idea is to have a cheap test, not a precise one : a bounding sphere will do : 
		double Angle = FMath::Max(Cone.Angle1, Cone.Angle2) * UE_PI / 180.0;
		double CosAngle = FMath::Cos(Angle);
		FVector SegmentStart = Cone.ConeToWorld.TransformPosition(FVector::ZeroVector);
		FVector SegmentEnd = Cone.ConeToWorld.TransformPosition(FVector::XAxisVector);
		FVector Forward = (SegmentEnd - SegmentStart);
		double ConeLength = Forward.Size();

		FVector SphereCenter;
		double Radius;
		if (Angle > UE_PI / 4.0)
		{
			SphereCenter = SegmentStart + CosAngle * Forward;
			Radius = FMath::Sin(Angle) * ConeLength;
		}
		else
		{
			SphereCenter = SegmentStart + Forward / (2.0 * CosAngle);
			Radius = ConeLength / (2.0 * CosAngle);
		}

		if (!bAllowFrustumCulling || SphereInFrustum(SphereCenter, Radius, AdjustedFrustum))
		{
			TArray<FVector> Verts;
			Cone.Draw(PDI, (Cone.DrawTypeOverride != EDrawType::Invalid) ? Cone.DrawTypeOverride : DrawType, DrawAlpha, DefaultMaterialCache, ViewIndex, Collector, &Verts);
		}
	}

	// Draw spheres
	for (const FSphere& Sphere : Spheres)
	{
		if (!bAllowFrustumCulling || SphereInFrustum(Sphere.Location, Sphere.Radius, AdjustedFrustum))
		{
			Sphere.Draw(PDI, (Sphere.DrawTypeOverride != EDrawType::Invalid) ? Sphere.DrawTypeOverride : DrawType, DrawAlpha, DefaultMaterialCache, ViewIndex, Collector);
		}
	}

	// Draw Capsules
	for (const FCapsule& Capsule : Capsules)
	{
		// The idea is to have cheap test, not a precise one : a bounding sphere will do : 
		const FVector Origin = Capsule.Base;
		const double HalfAxis = FMath::Max<double>(Capsule.HalfHeight - Capsule.Radius, 1.0);
		const FVector BottomEnd = Capsule.Base;
		const double TotalHalfLength = (HalfAxis + Capsule.Radius);
		const FVector TopEnd = Capsule.Base + 2.0 * TotalHalfLength * Capsule.Z;
		const FVector SphereCenter = (TopEnd + BottomEnd) / 2.0;

		if (!bAllowFrustumCulling || SphereInFrustum(SphereCenter, TotalHalfLength, AdjustedFrustum))
		{
			Capsule.Draw(PDI, (Capsule.DrawTypeOverride != EDrawType::Invalid) ? Capsule.DrawTypeOverride : DrawType, DrawAlpha, DefaultMaterialCache, ViewIndex, Collector);
		}
	}

	// Draw Meshes
	for (const FMesh& Mesh : Meshes)
	{
		if (!bAllowFrustumCulling || Mesh.Box.IsValid)
		{
			FVector Center, Extents;
			Mesh.Box.GetCenterAndExtents(Center, Extents);
			if (!BoxInFrustum(Center, Extents, AdjustedFrustum))
			{
				continue;
			}
		}
		FDynamicMeshBuilder MeshBuilder(View->GetFeatureLevel());
		MeshBuilder.AddVertices(Mesh.Vertices);
		MeshBuilder.AddTriangles(Mesh.Indices);

		FMaterialCache& MeshMaterialCache = Mesh.Color.A == 255 ? SolidMeshMaterialCache : DefaultMaterialCache;
		MeshBuilder.GetMesh(FMatrix::Identity, MeshMaterialCache[Mesh.Color], SDPG_World, false, false, ViewIndex, Collector);
	}

	// Draw Coordinate Systems
	for (const FCoordinateSystem CoordinateSystem : CoordinateSystems)
	{
		if (!bAllowFrustumCulling || SphereInFrustum(CoordinateSystem.AxisLoc, CoordinateSystem.Scale, AdjustedFrustum))
		{
			CoordinateSystem.Draw(PDI);
		}
	}
}

FConvexVolume FDebugRenderSceneProxy::AdjustViewFrustumForFarClipping(const FSceneView* InView, double InFarClippingDistance)
{
	if (InView == nullptr)
	{
		return FConvexVolume();
	}

	FConvexVolume AdjustedFrustum = InView->ViewFrustum;
	double FinalFarClippingDistance = InFarClippingDistance;
	if (const double OverrideFarClippingPlane = CVarDebugRenderOverrideFarClippingPlane.GetValueOnAnyThread(); OverrideFarClippingPlane > 0.0f)
	{ 
		FinalFarClippingDistance = OverrideFarClippingPlane;
	}

	if (FinalFarClippingDistance > 0.0f)
	{
		const FPlane FarPlane(InView->ViewMatrices.GetViewOrigin() + InView->GetViewDirection() * FinalFarClippingDistance, InView->GetViewDirection());
		// Derive the view frustum from the view projection matrix, overriding the far plane
		GetViewFrustumBounds(AdjustedFrustum, InView->ViewMatrices.GetWorldToClip(), FarPlane, true, false);
	}

	return AdjustedFrustum;
}

/**
* Draws a line with an arrow at the end.
*
* @param Start		Starting point of the line.
* @param End		Ending point of the line.
* @param Color		Color of the line.
* @param Mag		Size of the arrow.
*/
void FDebugRenderSceneProxy::DrawLineArrow(FPrimitiveDrawInterface* PDI,const FVector &Start,const FVector &End,const FColor &Color,float Mag) const
{
	const FArrowLine ArrowLine(Start, End, Color, Mag);
	ArrowLine.Draw(PDI);
}

FDebugRenderSceneProxy::FMaterialCache::FMaterialCache(FMeshElementCollector& InCollector, bool bUseLight, UMaterial* InMaterial)
	: Collector(InCollector)
	, SolidMeshMaterial(InMaterial)
	, bUseFakeLight(bUseLight)
{
}

FMaterialRenderProxy* FDebugRenderSceneProxy::FMaterialCache::operator[](FLinearColor Color)
{
	FMaterialRenderProxy* MeshColor = nullptr;
	const uint32 HashKey = GetTypeHashHelper(Color);
	if (MeshColorInstances.Contains(HashKey))
	{
		MeshColor = *MeshColorInstances.Find(HashKey);
	}
	else
	{
		if (bUseFakeLight && SolidMeshMaterial.IsValid())
		{
			MeshColor = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(
				SolidMeshMaterial->GetRenderProxy(),
				Color,
				"GizmoColor"
				);
		}
		else
		{
			MeshColor = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), Color);
		}

		MeshColorInstances.Add(HashKey, MeshColor);
	}

	return MeshColor;
}

void FDebugRenderSceneProxy::FDebugLine::Draw(FPrimitiveDrawInterface* PDI) const
{
	PDI->DrawLine(Start, End, Color, PriorityGroup, Thickness, 0, Thickness > 0);
}

void FDebugRenderSceneProxy::FWireStar::Draw(FPrimitiveDrawInterface* PDI) const
{
	DrawWireStar(PDI, Position, Size, Color, SDPG_World);
}

void FDebugRenderSceneProxy::FArrowLine::Draw(FPrimitiveDrawInterface* PDI) const
{
	// draw a pretty arrow
	FVector Dir = End - Start;
	const float DirMag = Dir.Size();
	Dir /= DirMag;
	FVector YAxis, ZAxis;
	Dir.FindBestAxisVectors(YAxis, ZAxis);
	FMatrix ArrowTM(Dir, YAxis, ZAxis, Start);
	DrawDirectionalArrow(PDI, ArrowTM, Color, DirMag, Mag, SDPG_World);
}

// Deprecated : set the Mag member and call the base Draw function instead
void FDebugRenderSceneProxy::FArrowLine::Draw(FPrimitiveDrawInterface* PDI, const float InMag) const
{
	// draw a pretty arrow
	FVector Dir = End - Start;
	const float DirMag = Dir.Size();
	Dir /= DirMag;
	FVector YAxis, ZAxis;
	Dir.FindBestAxisVectors(YAxis,ZAxis);
	FMatrix ArrowTM(Dir,YAxis,ZAxis,Start);
	DrawDirectionalArrow(PDI,ArrowTM,Color,DirMag,Mag,SDPG_World);
}

void FDebugRenderSceneProxy::FDashedLine::Draw(FPrimitiveDrawInterface* PDI) const
{
	DrawDashedLine(PDI, Start, End, Color, DashSize, SDPG_World);
}

void FDebugRenderSceneProxy::FDebugBox::Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	if (InDrawType == SolidAndWireMeshes || InDrawType == WireMesh)
	{
		DrawWireBox(PDI, Transform.ToMatrixWithScale(), Box, Color, SDPG_World, InDrawType == SolidAndWireMeshes ? 2 : Thickness, 0, true);
	}
	if (InDrawType == SolidAndWireMeshes || InDrawType == SolidMesh)
	{
		GetBoxMesh(FTransform(Box.GetCenter()).ToMatrixNoScale() * Transform.ToMatrixWithScale(), Box.GetExtent(), MaterialCache[Color.WithAlpha(InDrawAlpha * Color.A)], SDPG_World, ViewIndex, Collector);
	}
}

void FDebugRenderSceneProxy::FCircle::Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	FVector XAxis, YAxis;
	Axis.FindBestAxisVectors(XAxis, YAxis);
	if (InDrawType == SolidAndWireMeshes || InDrawType == WireMesh)
	{
		DrawCircle(PDI, Center, XAxis, YAxis, Color, Radius, (InDrawType == SolidAndWireMeshes) ? 9 : 12, SDPG_World, InDrawType == SolidAndWireMeshes ? 2 : Thickness, 0, true);
	}

	if (InDrawType == SolidAndWireMeshes || InDrawType == SolidMesh)
	{
		GetDiscMesh(Center, XAxis, YAxis, Radius, (InDrawType == SolidAndWireMeshes) ? 9 : 12, MaterialCache[Color.WithAlpha(InDrawAlpha * Color.A)], SDPG_World, ViewIndex, Collector);
	}
}

void FDebugRenderSceneProxy::FWireCylinder::Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	FVector XAxis, YAxis;
	Direction.FindBestAxisVectors(XAxis, YAxis);
	if (InDrawType == SolidAndWireMeshes || InDrawType == WireMesh)
	{
		DrawWireCylinder(PDI, Base, XAxis, YAxis, Direction, Color, Radius, HalfHeight, (InDrawType == SolidAndWireMeshes) ? 9 : 16, SDPG_World, InDrawType == SolidAndWireMeshes ? 2 : 0, 0, true);
	}

	if (InDrawType == SolidAndWireMeshes || InDrawType == SolidMesh)
	{
		GetCylinderMesh(Base, XAxis, YAxis, Direction, Radius, HalfHeight, (InDrawType == SolidAndWireMeshes) ? 9 : 16, MaterialCache[Color.WithAlpha(InDrawAlpha * Color.A)], SDPG_World, ViewIndex, Collector);
	}
}

void FDebugRenderSceneProxy::FCone::Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector, TArray<FVector>* VertsCache) const
{
	if (InDrawType == SolidAndWireMeshes || InDrawType == WireMesh)
	{
		TArray<FVector> LocalVertsCache;
		TArray<FVector>& Verts = VertsCache != nullptr ? *VertsCache : LocalVertsCache;
		DrawWireCone(PDI, Verts, ConeToWorld, 1, Angle2, (InDrawType == SolidAndWireMeshes) ? 9 : 16, Color, SDPG_World, InDrawType == SolidAndWireMeshes ? 2 : 0, 0, true);
	}
	if (InDrawType == SolidAndWireMeshes || InDrawType == SolidMesh)
	{
		GetConeMesh(ConeToWorld, Angle1, Angle2, (InDrawType == SolidAndWireMeshes) ? 9 : 16, MaterialCache[Color.WithAlpha(InDrawAlpha * Color.A)], SDPG_World, ViewIndex, Collector);
	}
}


void FDebugRenderSceneProxy::FSphere::Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	if (InDrawType == SolidAndWireMeshes || InDrawType == WireMesh)
	{
		DrawWireSphere(PDI, Location, Color.WithAlpha(255), Radius, 20, SDPG_World, InDrawType == SolidAndWireMeshes ? 2 : 0, 0, true);
	}
	if (InDrawType == SolidAndWireMeshes || InDrawType == SolidMesh)
	{
		GetSphereMesh(Location, FVector(Radius), 20, 7, MaterialCache[Color.WithAlpha(InDrawAlpha * Color.A)], SDPG_World, false, ViewIndex, Collector);
	}
}

void FDebugRenderSceneProxy::FCapsule::Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	if (InDrawType == SolidAndWireMeshes || InDrawType == WireMesh)
	{
		const float HalfAxis = FMath::Max<float>(HalfHeight - Radius, 1.f);
		const FVector BottomEnd = Base + Radius * Z;
		const FVector TopEnd = BottomEnd + (2 * HalfAxis) * Z;
		const float CylinderHalfHeight = (TopEnd - BottomEnd).Size() * 0.5;
		const FVector CylinderLocation = BottomEnd + CylinderHalfHeight * Z;
		DrawWireCapsule(PDI, CylinderLocation, X, Y, Z, Color, Radius, HalfHeight, (InDrawType == SolidAndWireMeshes) ? 9 : 16, SDPG_World, InDrawType == SolidAndWireMeshes ? 2 : 0, 0, true);
	}
	if (InDrawType == SolidAndWireMeshes || InDrawType == SolidMesh)
	{
		GetCapsuleMesh(Base, X, Y, Z, Color, Radius, HalfHeight, (InDrawType == SolidAndWireMeshes) ? 9 : 16, MaterialCache[Color.WithAlpha(InDrawAlpha * Color.A)], SDPG_World, false, ViewIndex, Collector);
	}
}

void FDebugRenderSceneProxy::FCoordinateSystem::Draw(FPrimitiveDrawInterface* PDI) const
{
	DrawCoordinateSystem(PDI, AxisLoc, AxisRot, Scale, SDPG_World, Thickness);
}