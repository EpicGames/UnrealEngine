// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugViewModeHelpers.cpp: debug view shader helpers.
=============================================================================*/

#include "DebugViewModeHelpers.h"

#include "ActorEditorUtils.h"
#include "Components/PrimitiveComponent.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DebugViewModeInterface.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "MeshMaterialShader.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/FeedbackContext.h"
#include "RenderingThread.h"
#include "ShaderCompiler.h"
#include "RenderUtils.h"

#define LOCTEXT_NAMESPACE "LogDebugViewMode"

const TCHAR* DebugViewShaderModeToString(EDebugViewShaderMode InShaderMode)
{
	switch (InShaderMode)
	{
	case DVSM_None:
		return TEXT("DVSM_None");
	case DVSM_ShaderComplexity:
		return TEXT("DVSM_ShaderComplexity");
	case DVSM_ShaderComplexityContainedQuadOverhead:
		return TEXT("DVSM_ShaderComplexityContainedQuadOverhead");
	case DVSM_ShaderComplexityBleedingQuadOverhead:
		return TEXT("DVSM_ShaderComplexityBleedingQuadOverhead");
	case DVSM_QuadComplexity:
		return TEXT("DVSM_QuadComplexity");
	case DVSM_PrimitiveDistanceAccuracy:
		return TEXT("DVSM_PrimitiveDistanceAccuracy");
	case DVSM_MeshUVDensityAccuracy:
		return TEXT("DVSM_MeshUVDensityAccuracy");
	case DVSM_MaterialTextureScaleAccuracy:
		return TEXT("DVSM_MaterialTextureScaleAccuracy");
	case DVSM_OutputMaterialTextureScales:
		return TEXT("DVSM_OutputMaterialTextureScales");
	case DVSM_RequiredTextureResolution:
		return TEXT("DVSM_RequiredTextureResolution");
	case DVSM_LODColoration:
		return TEXT("DVSM_LODColoration");
	case DVSM_VisualizeGPUSkinCache:
		return TEXT("DVSM_VisualizeGPUSkinCache");
	case DVSM_LWCComplexity:
		return TEXT("DVSM_LWCComplexity");
	case DVSM_ShadowCasters:
		return TEXT("DVSM_ShadowCasters");
	case DVSM_StreamingDeficit:
		return TEXT("DVSM_StreamingDeficit");
	default:
		return TEXT("DVSM_None");
	}
}

#if WITH_DEBUG_VIEW_MODES

bool IsDebugViewShaderModeODSCOnly()
{
	// If we return true here then the debug mode shaders would _only_ compile through ODSC.
	// This would save on always compiling the debug shader types for any loaded materials in render commandlets (where ODSC is off).
	// BUT the DVSM_OutputMaterialTextureScales mode _is_ used in commandlets to build texture streaming data, so we can't take advantage of this (yet).
	return false;
}

// UE_DEPRECATED(5.8, "SupportDebugViewModes is deprecated, please use ForceEnableDebugViewSupport/ForceDisableDebugViewSupport instead.")
bool SupportDebugViewModes()
{
	// Allow command line disable of debug view modes.
	// This is intended only for a very specific commandlet case where we can demand debug view shaders through ODSC even when they are not actually needed.
	// Setting this command line avoids _any_ potential overhead from debug shader compilation.
	static const bool bForceDisableDebugViewModes = FParse::Param(FCommandLine::Get(), TEXT("forceDisableDebugViewModes"));
	return !bForceDisableDebugViewModes;
}


/** Returns true if we are forcing debug view support even for non editor or ODSC use cases. */
bool ForceEnableDebugViewSupport(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.ForceDebugViewModes"));
	return CVar.Get(Platform) == 1;
}

/** Returns true if we are force disabling debug view support even for editor use cases. */
bool ForceDisableDebugViewSupport(EShaderPlatform Platform)
{
	// Allow command line disable of debug view modes.
	// This is intended only for a very specific commandlet case where we can demand debug view shaders through ODSC even when they are not actually needed.
	// Setting this command line avoids _any_ potential overhead from debug shader compilation.
	static const bool bForceDisableDebugViewModes = FParse::Param(FCommandLine::Get(), TEXT("forceDisableDebugViewModes"));
	return bForceDisableDebugViewModes;
}

bool SupportDebugViewShaderMode(EDebugViewShaderMode ShaderMode, EShaderPlatform Platform)
{
	if (ForceDisableDebugViewSupport(Platform))
	{
		return false;
	}

	// Quad modes don't compile/work on Metal < SM6 or Playstation. 
	const bool bSupportQuadComplexityModes =
		!FDataDrivenShaderPlatformInfo::GetIsLanguageSony(Platform) && 
		(!FDataDrivenShaderPlatformInfo::GetIsLanguageMetal(Platform) || IsMetalSM6Platform(Platform));
	
	switch (ShaderMode)
	{
	case DVSM_LODColoration:
	case DVSM_VisualizeGPUSkinCache:
	case DVSM_StreamingDeficit:
		return true;

	case DVSM_ShadowCasters:
		return DoesPlatformSupportVirtualShadowMaps(Platform);

	case DVSM_QuadComplexity:
		return bSupportQuadComplexityModes;

#if WITH_EDITORONLY_DATA
	// All shader complexity modes requires editor data.
	case DVSM_ShaderComplexity:
	case DVSM_LWCComplexity:
		return true;
	case DVSM_ShaderComplexityContainedQuadOverhead:
	case DVSM_ShaderComplexityBleedingQuadOverhead:
		return bSupportQuadComplexityModes;

	// All texture streaming modes require editor data and are not supported on mobile.
	case DVSM_PrimitiveDistanceAccuracy:
	case DVSM_MeshUVDensityAccuracy:
	case DVSM_MaterialTextureScaleAccuracy:
	case DVSM_RequiredTextureResolution:
	case DVSM_OutputMaterialTextureScales:
		return !IsMobilePlatform(Platform);
#endif

	default:
		return false;
	}
}

bool AllowDebugViewShaderMode(EDebugViewShaderMode ShaderMode, EShaderPlatform Platform, ERHIFeatureLevel::Type FeatureLevel)
{
	// Only allow debug modes which are supported.
	if (!SupportDebugViewShaderMode(ShaderMode, Platform))
	{
		return false;
	}

	if (ForceEnableDebugViewSupport(Platform))
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	bool bRequireODSCShaders = IsDebugViewShaderModeODSCOnly();
#else
	bool bRequireODSCShaders = true;
#endif

	// Don't allow debug modes if we require ODSC but can't compile ODSC shaders.
	if (bRequireODSCShaders && !ShouldCompileODSCOnlyShaders())
	{
		return false;
	}

	return true;
}

#endif // WITH_DEBUG_VIEW_MODES


int32 GetNumActorsInWorld(UWorld* InWorld)
{
	int32 ActorCount = 0;
	for (int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels(); LevelIndex++)
	{
		ULevel* Level = InWorld->GetLevel(LevelIndex);
		if (!Level)
		{
			continue;
		}

		ActorCount += Level->Actors.Num();
	}
	return ActorCount;
}

bool WaitForShaderCompilation(const FText& Message, FSlowTask* ProgressTask)
{
	FlushRenderingCommands();

	const int32 NumShadersToBeCompiled = GShaderCompilingManager->GetNumRemainingJobs();
	int32 RemainingShaders = NumShadersToBeCompiled;
	if (NumShadersToBeCompiled > 0)
	{
		FScopedSlowTask SlowTask(1.f, Message);

		while (RemainingShaders > 0)
		{
			FPlatformProcess::Sleep(0.01f);
			GShaderCompilingManager->ProcessAsyncResults(false, true);

			const int32 RemainingShadersThisFrame = GShaderCompilingManager->GetNumRemainingJobs();
			if (RemainingShadersThisFrame > 0)
			{
				const int32 NumberOfShadersCompiledThisFrame = RemainingShaders - RemainingShadersThisFrame;

				const float FrameProgress = (float)NumberOfShadersCompiledThisFrame / (float)NumShadersToBeCompiled;
				if (ProgressTask)
				{
					ProgressTask->EnterProgressFrame(FrameProgress);
					SlowTask.EnterProgressFrame(FrameProgress);
					if (GWarn->ReceivedUserCancel())
					{
						return false;
					}
				}
			}
			RemainingShaders = RemainingShadersThisFrame;
		}
	}
	else if (ProgressTask)
	{
		ProgressTask->EnterProgressFrame();
		if (GWarn->ReceivedUserCancel())
		{
			return false;
		}
	}

	// Extra safety to make sure every shader map is updated
	GShaderCompilingManager->FinishAllCompilation();
	FlushRenderingCommands();

	return true;
}

/** Get the list of all material used in a world 
 *
 * @return true if the operation is a success, false if it was canceled.
 */
bool GetUsedMaterialsInWorld(UWorld* InWorld, OUT TSet<UMaterialInterface*>& OutMaterials, FSlowTask* ProgressTask)
{
#if WITH_EDITORONLY_DATA
	if (!InWorld)
	{
		return false;
	}

	const int32 NumActorsInWorld = GetNumActorsInWorld(InWorld);
	if (!NumActorsInWorld)
	{
		if (ProgressTask)
		{
			ProgressTask->EnterProgressFrame();
		}
		return true;
	}

	const float OneOverNumActorsInWorld = 1.f / (float)NumActorsInWorld;

	FScopedSlowTask SlowTask(1.f, (LOCTEXT("TextureStreamingBuild_GetTextureStreamingBuildMaterials", "Getting materials to rebuild")));

	for (int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels(); LevelIndex++)
	{
		ULevel* Level = InWorld->GetLevel(LevelIndex);
		if (!Level)
		{
			continue;
		}

		for (AActor* Actor : Level->Actors)
		{
			if (ProgressTask)
			{
				ProgressTask->EnterProgressFrame(OneOverNumActorsInWorld);
				SlowTask.EnterProgressFrame(OneOverNumActorsInWorld);
				if (GWarn->ReceivedUserCancel())
				{
					return false;
				}
			}

			// Check the actor after incrementing the progress.
			if (!Actor || FActorEditorUtils::IsABuilderBrush(Actor))
			{
				continue;
			}

			TInlineComponentArray<UPrimitiveComponent*> Primitives;
			Actor->GetComponents(Primitives);

			for (UPrimitiveComponent* Primitive : Primitives)
			{
				if (!Primitive)
				{
					continue;
				}

				TArray<UMaterialInterface*> Materials;
				Primitive->GetUsedMaterials(Materials);

				for (UMaterialInterface* Material : Materials)
				{
					if (Material)
					{
						OutMaterials.Add(Material);
					}
				}
			}
		}
	}
	return OutMaterials.Num() != 0;
#else
	return false;
#endif
}

/**
 * Build Shaders to compute scales per texture.
 *
 * @param QualityLevel		The quality level for the shaders.
 * @param FeatureLevel		The feature level for the shaders.
 * @param Materials			The materials to update, the one that failed compilation will be removed (IN OUT).
 * @return true if the operation is a success, false if it was canceled.
 */
bool CompileDebugViewModeShaders(EDebugViewShaderMode ShaderMode, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<UMaterialInterface*>& Materials, FSlowTask* ProgressTask)
{
#if WITH_EDITORONLY_DATA
	if (!GShaderCompilingManager || !Materials.Num())
	{
		return false;
	}

	const FDebugViewModeInterface* DebugViewModeInterface = FDebugViewModeInterface::GetInterface(ShaderMode);
	if (!DebugViewModeInterface)
	{
		return false;
	}

	const FVertexFactoryType* LocalVertexFactory = FindVertexFactoryType(TEXT("FLocalVertexFactory"));
	if (!LocalVertexFactory)
	{
		return false;
	}
	
	EShaderPlatform Platform = GetFeatureLevelShaderPlatform(FeatureLevel);

	FMaterialShaderTypes ShaderTypes;
	DebugViewModeInterface->AddShaderTypes(FeatureLevel, LocalVertexFactory, ShaderTypes);

	TArray<FMaterial*> PendingMaterials;
	PendingMaterials.Reserve(Materials.Num());

	for (TSet<UMaterialInterface*>::TIterator It(Materials); It; ++It)
	{
		UMaterialInterface* MaterialInterface = *It;
		check(MaterialInterface); // checked for null in GetTextureStreamingBuildMaterials
		
		FMaterial* Material = MaterialInterface->GetMaterialResource(Platform, QualityLevel);
		if (!Material)
		{
			continue;
		}

		// Remove materials incompatible with debug view modes (e.g. landscape materials can only be compiled with the landscape VF)
		if (Material->GetMaterialDomain() != MD_Surface || Material->IsUsedWithLandscape())
		{
			It.RemoveCurrent();
			continue;
		}
		
		// If material needs the shaders for this platform cached, begin the operation.
		if (Material->GetGameThreadShaderMap()
			&& Material->ShouldCacheShaders(ShaderTypes, LocalVertexFactory)
			&& !Material->HasShaders(ShaderTypes, LocalVertexFactory))
		{
			Material->CacheShaders(EMaterialShaderPrecompileMode::Default);
			PendingMaterials.Push(Material);
		}
	}

	bool bAllMaterialsCompiledSuccessfully = true;
	while (PendingMaterials.Num() > 0)
	{
		FMaterial* Material = PendingMaterials.Last();

		// Check if material has completed compiling the shaders.
		if (Material->IsCompilationFinished())
		{
			bAllMaterialsCompiledSuccessfully &= Material->HasShaders(ShaderTypes, LocalVertexFactory);
			PendingMaterials.Pop();
			continue;
		}

		// Are we asked to cancel the operation?
		ProgressTask->TickProgress();
		if (GWarn->ReceivedUserCancel())
		{
			bAllMaterialsCompiledSuccessfully = false;
			break;
		}

		// Wait a little then try again.
		FPlatformProcess::Sleep(0.1f);
		GShaderCompilingManager->ProcessAsyncResults(false, false);
	}

	return bAllMaterialsCompiledSuccessfully;

#else
	return false;
#endif

}

#undef LOCTEXT_NAMESPACE
