// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DebugViewModeInterface.cpp: Contains definitions for rendering debug viewmodes.
=============================================================================*/

#include "DebugViewModeInterface.h"

#if ENABLE_DRAW_DEBUG

#include "RHIStaticStates.h"
#include "MaterialShared.h"

FDebugViewModeInterface* FDebugViewModeInterface::Singleton = nullptr;

void FDebugViewModeInterface::SetDrawRenderState(EDebugViewShaderMode DebugViewMode, EBlendMode InBlendMode, FRenderState& DrawRenderState, bool bHasDepthPrepassForMaskedMaterial, bool bDitheredLODTransition) const
{
	// No writes to RT1
	FBlendStateRHIRef DefaultOpaqueBlendState = TStaticBlendState<
			/* RT0ColorWriteMask */ CW_RGBA,
			/* RT0ColorBlendOp   */ BO_Add,
			/* RT0ColorSrcBlend  */ BF_One,
			/* RT0ColorDestBlend */ BF_Zero,
			/* RT0AlphaBlendOp   */ BO_Add,
			/* RT0AlphaSrcBlend  */ BF_One,
			/* RT0AlphaDestBlend */ BF_Zero,
			/* RT1ColorWriteMask */ CW_NONE
		>::GetRHI();

	// Default values
	if (IsTranslucentBlendMode(InBlendMode))
	{
		// shaders will use an hardcoded alpha
		DrawRenderState.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha, CW_NONE>::GetRHI();
		DrawRenderState.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
	}
	else
	{
		DrawRenderState.BlendState = DefaultOpaqueBlendState;

		// If not selected, use depth equal to make alpha test stand out (goes with EarlyZPassMode = DDM_AllOpaque) 
		if (IsMaskedBlendMode(InBlendMode) && bHasDepthPrepassForMaskedMaterial)
		{
			DrawRenderState.DepthStencilState = TStaticDepthStencilState<false, CF_Equal>::GetRHI();
		}
		else
		{
			DrawRenderState.DepthStencilState = TStaticDepthStencilState<>::GetRHI();
		}
	}

	// Viewmode overrides
	if (DebugViewMode == DVSM_QuadComplexity 
		|| DebugViewMode == DVSM_ShaderComplexityBleedingQuadOverhead 
		|| DebugViewMode == DVSM_ShaderComplexityContainedQuadOverhead 
		|| DebugViewMode == DVSM_ShaderComplexity)
	{
		if (IsOpaqueBlendMode(InBlendMode))
		{
			DrawRenderState.DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
		}
		else if (IsMaskedBlendMode(InBlendMode))
		{
			if (bHasDepthPrepassForMaskedMaterial)
			{
				DrawRenderState.DepthStencilState = TStaticDepthStencilState<false, CF_Equal>::GetRHI();
			}
			else
			{
				DrawRenderState.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
			}
		}
		else // Translucent
		{
			DrawRenderState.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
		}
		
		DrawRenderState.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_One, CW_NONE>::GetRHI();
	}
	else if (DebugViewMode == DVSM_OutputMaterialTextureScales)
	{
		DrawRenderState.BlendState = TStaticBlendState<>::GetRHI();
		DrawRenderState.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	}
	else if (DebugViewMode == DVSM_ShadowCasters)
	{
		// enable RT 1
		if (IsTranslucentBlendMode(InBlendMode))
		{
			DrawRenderState.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha, CW_RED>::GetRHI();
		}
		else
		{
			DrawRenderState.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero, CW_RED>::GetRHI();
		}
	}
	else if (DebugViewMode == DVSM_LODColoration && bDitheredLODTransition)
	{
		DrawRenderState.DepthStencilState = TStaticDepthStencilState<false, CF_Equal>::GetRHI();
	}
}

void FDebugViewModeInterface::SetInterface(FDebugViewModeInterface* Interface)
{
	ensure(!Singleton);
	Singleton = Interface;
}

bool FDebugViewModeInterface::AllowFallbackToDefaultMaterial(bool bHasVertexPositionOffsetConnected, bool bHasPixelDepthOffsetConnected)
{
	// Check for anything that could change the shape from the default material.
	return !bHasVertexPositionOffsetConnected &&
		!bHasPixelDepthOffsetConnected;
}

bool FDebugViewModeInterface::AllowFallbackToDefaultMaterial(const FMaterial* InMaterial)
{
	check(InMaterial);
	return FDebugViewModeInterface::AllowFallbackToDefaultMaterial(InMaterial->HasVertexPositionOffsetConnected(), InMaterial->HasPixelDepthOffsetConnected());
}


#endif // ENABLE_DRAW_DEBUG

