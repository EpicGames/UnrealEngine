// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DecalActor.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Materials/MaterialInterface.h"
#include "Engine/Texture2D.h"
#include "Components/BoxComponent.h"
#include "Components/DecalComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DecalActor)


#if WITH_EDITOR
namespace DecalEditorConstants
{
	/** Scale factor to apply to get nice scaling behaviour in-editor when using percentage-based scaling */
	static const float PercentageScalingMultiplier = 5.0f;

	/** Scale factor to apply to get nice scaling behaviour in-editor when using additive-based scaling */
	static const float AdditiveScalingMultiplier = 50.0f;
}
#endif

ADecalActor::ADecalActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Decal = CreateDefaultSubobject<UDecalComponent>(TEXT("NewDecalComponent"));
	Decal->SetRelativeRotation(FRotator(-90, 0, 0));
	Decal->bDestroyOwnerAfterFade = true;

	RootComponent = Decal;

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> DecalTexture;
			FName ID_Decals;
			FText NAME_Decals;
			FConstructorStatics()
				: DecalTexture(TEXT("/Engine/EditorResources/S_DecalActorIcon"))
				, ID_Decals(TEXT("Decals"))
				, NAME_Decals(NSLOCTEXT("SpriteCategory", "Decals", "Decals"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (ArrowComponent)
		{
			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->ArrowSize = 1.0f;
			ArrowComponent->ArrowColor = FColor(80, 80, 200, 255);
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Decals;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Decals;
			ArrowComponent->SetupAttachment(Decal);
			ArrowComponent->SetUsingAbsoluteScale(true);
			ArrowComponent->bIsScreenSizeScaled = true;
		}

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.DecalTexture.Get();
			SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Decals;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Decals;
			SpriteComponent->SetupAttachment(Decal);
			SpriteComponent->bIsScreenSizeScaled = true;
			SpriteComponent->SetUsingAbsoluteScale(true);
			SpriteComponent->bReceivesDecals = false;
		}
	}
#endif // WITH_EDITORONLY_DATA

	SetCanBeDamaged(false);
}

#if WITH_EDITOR
void ADecalActor::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	if (Decal)
	{
		Decal->RecreateRenderState_Concurrent();
	}
}

void ADecalActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// AActor::PostEditChange will ForceUpdateComponents()
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (Decal)
	{
		Decal->RecreateRenderState_Concurrent();
	}
}

// @return 1 / x or 0 if the result would be a NaN
static float SaveInv(float x)
{
	return FMath::IsNearlyZero(x) ? 0.0f : (1.0f / x);
}

void ADecalActor::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	FVector Mul(1.0f, 1.0f, 1.0f);

	if(Decal)
	{
		Mul = FVector(SaveInv(Decal->DecalSize.X), SaveInv(Decal->DecalSize.Y), SaveInv(Decal->DecalSize.Z));
	}

	const FVector ModifiedScale = Mul * DeltaScale * (AActor::bUsePercentageBasedScaling ? DecalEditorConstants::PercentageScalingMultiplier : DecalEditorConstants::AdditiveScalingMultiplier);

	Super::EditorApplyScale(ModifiedScale, PivotLocation, bAltDown, bShiftDown, bCtrlDown);
}

bool ADecalActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	UMaterialInterface* DecalMaterial = Decal->GetDecalMaterial();
	if (DecalMaterial)
	{
		Objects.Add(DecalMaterial);
	}

	return true;
}

#endif // WITH_EDITOR

void ADecalActor::SetDecalMaterial(class UMaterialInterface* NewDecalMaterial)
{
	if (Decal)
	{
		Decal->SetDecalMaterial(NewDecalMaterial);
	}
}

class UMaterialInterface* ADecalActor::GetDecalMaterial() const
{
	return Decal ? Decal->GetDecalMaterial() : NULL;
}

class UMaterialInstanceDynamic* ADecalActor::CreateDynamicMaterialInstance()
{
	return Decal ? Decal->CreateDynamicMaterialInstance() : NULL;
}

void ADecalActor::Serialize(FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_DECAL_SIZE)
	{
		if(Decal)
		{
			// before Super::Serialize(Ar);
			Decal->SetRelativeScale3D_Direct(FVector(128.0f, 256.0f, 256.0f));
		}
	}

	Super::Serialize(Ar);

	if (Ar.UEVer() < VER_UE4_DECAL_SIZE)
	{
		if(Decal)
		{
			// after Super::Serialize(Ar);
			Decal->DecalSize = FVector(1.0f, 1.0f, 1.0f);
		}
	}
}

void ADecalActor::PostLoad()
{
	 Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if(BoxComponent_DEPRECATED)
	{
		// formerly we used this component to draw a box, now we use the DecalComponentVisualizer
		BoxComponent_DEPRECATED->DestroyComponent();
		BoxComponent_DEPRECATED = 0;
	}
#endif
}


