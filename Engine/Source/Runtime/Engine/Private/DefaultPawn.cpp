// Copyright Epic Games, Inc. All Rights Reserved.


#include "GameFramework/DefaultPawn.h"
#include "GameFramework/Controller.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/InputComponent.h"
#include "Engine/World.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "Components/SphereComponent.h"
#include "GameFramework/FloatingPawnMovement.h"
#include "GameFramework/PlayerInput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultPawn)

FName ADefaultPawn::MovementComponentName(TEXT("MovementComponent0"));
FName ADefaultPawn::CollisionComponentName(TEXT("CollisionComponent0"));
FName ADefaultPawn::MeshComponentName(TEXT("MeshComponent0"));


ADefaultPawn::ADefaultPawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCanBeDamaged(true);

	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	NetPriority = 3.0f;

	BaseEyeHeight = 0.0f;
	bCollideWhenPlacing = false;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

	CollisionComponent = CreateDefaultSubobject<USphereComponent>(ADefaultPawn::CollisionComponentName);
	CollisionComponent->InitSphereRadius(35.0f);
	CollisionComponent->SetCollisionProfileName(UCollisionProfile::Pawn_ProfileName);

	CollisionComponent->CanCharacterStepUpOn = ECB_No;
	CollisionComponent->SetShouldUpdatePhysicsVolume(true);
	CollisionComponent->SetCanEverAffectNavigation(false);
	CollisionComponent->bDynamicObstacle = true;

	RootComponent = CollisionComponent;

	MovementComponent = CreateDefaultSubobject<UPawnMovementComponent, UFloatingPawnMovement>(ADefaultPawn::MovementComponentName);
	MovementComponent->UpdatedComponent = CollisionComponent;

	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UStaticMesh> SphereMesh;
		FConstructorStatics()
			: SphereMesh(TEXT("/Engine/EngineMeshes/Sphere")) {}
	};

	static FConstructorStatics ConstructorStatics;

	MeshComponent = CreateOptionalDefaultSubobject<UStaticMeshComponent>(ADefaultPawn::MeshComponentName);
	if (MeshComponent)
	{
		MeshComponent->SetCanEverAffectNavigation(false);
		MeshComponent->SetStaticMesh(ConstructorStatics.SphereMesh.Object);
		MeshComponent->AlwaysLoadOnClient = true;
		MeshComponent->AlwaysLoadOnServer = true;
		MeshComponent->bOwnerNoSee = true;
		MeshComponent->bCastDynamicShadow = true;
		MeshComponent->bAffectDynamicIndirectLighting = false;
		MeshComponent->bAffectDistanceFieldLighting = false;
		MeshComponent->bVisibleInRayTracing = false;
		MeshComponent->PrimaryComponentTick.TickGroup = TG_PrePhysics;
		MeshComponent->SetupAttachment(RootComponent);
		MeshComponent->SetCollisionProfileName(UCollisionProfile::Pawn_ProfileName);
		const float Scale = CollisionComponent->GetUnscaledSphereRadius() / 160.f; // @TODO: hardcoding known size of EngineMeshes.Sphere. Should use a unit sphere instead.
		MeshComponent->SetRelativeScale3D(FVector(Scale));
		MeshComponent->SetGenerateOverlapEvents(false);
	}

	// This is the default pawn class, we want to have it be able to move out of the box.
	bAddDefaultMovementBindings = true;

	BaseTurnRate = 45.f;
	BaseLookUpRate = 45.f;
}

void InitializeDefaultPawnInputBindings()
{
	static bool bBindingsAdded = false;
	if (!bBindingsAdded)
	{
		bBindingsAdded = true;

		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveForward", EKeys::W, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveForward", EKeys::S, -1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveForward", EKeys::Up, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveForward", EKeys::Down, -1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveForward", EKeys::Gamepad_LeftY, 1.f));

		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveRight", EKeys::A, -1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveRight", EKeys::D, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveRight", EKeys::Gamepad_LeftX, 1.f));

		// HACK: Android controller bindings in ini files seem to not work
		//  Direct overrides here some to work
#if !PLATFORM_ANDROID
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::Gamepad_LeftThumbstick, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::Gamepad_RightThumbstick, -1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::Gamepad_FaceButton_Bottom, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::LeftControl, -1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::SpaceBar, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::C, -1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::E, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::Q, -1.f));
#else
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::Gamepad_LeftTriggerAxis, -0.5f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_MoveUp", EKeys::Gamepad_RightTriggerAxis, 0.5f));
#endif

		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_TurnRate", EKeys::Gamepad_RightX, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_TurnRate", EKeys::Left, -1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_TurnRate", EKeys::Right, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_Turn", EKeys::MouseX, 1.f));

		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_LookUpRate", EKeys::Gamepad_RightY, 1.f));
		UPlayerInput::AddEngineDefinedAxisMapping(FInputAxisKeyMapping("DefaultPawn_LookUp", EKeys::MouseY, -1.f));
	}
}

void ADefaultPawn::SetupPlayerInputComponent(UInputComponent* PlayerInputComponent)
{
	check(PlayerInputComponent);

	if (bAddDefaultMovementBindings)
	{
		InitializeDefaultPawnInputBindings();

		PlayerInputComponent->BindAxis("DefaultPawn_MoveForward", this, &ADefaultPawn::MoveForward);
		PlayerInputComponent->BindAxis("DefaultPawn_MoveRight", this, &ADefaultPawn::MoveRight);
		PlayerInputComponent->BindAxis("DefaultPawn_MoveUp", this, &ADefaultPawn::MoveUp_World);
		PlayerInputComponent->BindAxis("DefaultPawn_Turn", this, &ADefaultPawn::AddControllerYawInput);
		PlayerInputComponent->BindAxis("DefaultPawn_TurnRate", this, &ADefaultPawn::TurnAtRate);
		PlayerInputComponent->BindAxis("DefaultPawn_LookUp", this, &ADefaultPawn::AddControllerPitchInput);
		PlayerInputComponent->BindAxis("DefaultPawn_LookUpRate", this, &ADefaultPawn::LookUpAtRate);
	}
}

void ADefaultPawn::UpdateNavigationRelevance()
{
	if (CollisionComponent)
	{
		CollisionComponent->SetCanEverAffectNavigation(bCanAffectNavigationGeneration);
	}
}

void ADefaultPawn::MoveRight(float Val)
{
	if (Val != 0.f)
	{
		if (GetController())
		{
			FRotator const ControlSpaceRot = GetController()->GetControlRotation();

			// transform to world space and add it
			AddMovementInput( FRotationMatrix(ControlSpaceRot).GetScaledAxis( EAxis::Y ), Val );
		}
	}
}

void ADefaultPawn::MoveForward(float Val)
{
	if (Val != 0.f)
	{
		if (GetController())
		{
			FRotator const ControlSpaceRot = GetController()->GetControlRotation();

			// transform to world space and add it
			AddMovementInput( FRotationMatrix(ControlSpaceRot).GetScaledAxis( EAxis::X ), Val );
		}
	}
}

void ADefaultPawn::MoveUp_World(float Val)
{
	if (Val != 0.f)
	{
		AddMovementInput(FVector::UpVector, Val);
	}
}

void ADefaultPawn::TurnAtRate(float Rate)
{
	// calculate delta for this frame from the rate information
	AddControllerYawInput(Rate * BaseTurnRate * GetWorld()->GetDeltaSeconds() * CustomTimeDilation);
}

void ADefaultPawn::LookUpAtRate(float Rate)
{
	// calculate delta for this frame from the rate information
	AddControllerPitchInput(Rate * BaseLookUpRate * GetWorld()->GetDeltaSeconds() * CustomTimeDilation);
}

UPawnMovementComponent* ADefaultPawn::GetMovementComponent() const
{
	return MovementComponent;
}


