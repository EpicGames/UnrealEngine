// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/DefaultPhysicsVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultPhysicsVolume)

ADefaultPhysicsVolume::ADefaultPhysicsVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

#if WITH_EDITORONLY_DATA
	// Not allowed to be selected or edited within Unreal Editor
	bEditable = false;
#endif // WITH_EDITORONLY_DATA

	// update default values when world is restarted
	TerminalVelocity = UPhysicsSettings::Get()->DefaultTerminalVelocity;
	FluidFriction = UPhysicsSettings::Get()->DefaultFluidFriction;

	// DefaultPhysicsVolumes are spawned only as a fallback object when determining the current physics volume.
	// They are not intended to actually have any collision response, as they don't have actual collision geometry.
	GetBrushComponent()->SetCollisionEnabled(ECollisionEnabled::NoCollision);
}

