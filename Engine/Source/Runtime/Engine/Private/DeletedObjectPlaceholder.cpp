// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeletedObjectPlaceholder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeletedObjectPlaceholder)

#if WITH_EDITOR

#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"

UDeletedObjectPlaceholder::FObjectCreated UDeletedObjectPlaceholder::OnObjectCreated;

bool UDeletedObjectPlaceholder::IsAsset() const
{
	return IsPackageExternal() && !GetPackage()->HasAnyFlags(RF_Transient) && !HasAnyFlags(RF_ClassDefaultObject) && !GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor) && !AActor::FindActorInPackage(GetExternalPackage());
}

UDeletedObjectPlaceholder* UDeletedObjectPlaceholder::Create(UObject* InOuter, UPackage* InPackage, const UObject* InOriginalObject)
{
	if (IsRunningCommandlet())
	{
		return nullptr;
	}
	check(InOuter);
	check(InOriginalObject);
	check(!UDeletedObjectPlaceholder::FindInPackage(InPackage));
	const AActor* OriginalActor = Cast<AActor>(InOriginalObject);
	UDeletedObjectPlaceholder* DeletedObjectPlaceholder = NewObject<UDeletedObjectPlaceholder>(InOuter, NAME_None, RF_Standalone | RF_Transactional | RF_Transient);
	DeletedObjectPlaceholder->SetExternalPackage(InPackage);
	DeletedObjectPlaceholder->OriginalObject = InOriginalObject;
	DeletedObjectPlaceholder->DisplayName = OriginalActor ? OriginalActor->GetActorLabel() : InOriginalObject->GetName();
	DeletedObjectPlaceholder->ExternalDataLayerUID = OriginalActor && OriginalActor->GetExternalDataLayerAsset() ? OriginalActor->GetExternalDataLayerAsset()->GetUID() : FExternalDataLayerUID();

	OnObjectCreated.Broadcast(DeletedObjectPlaceholder);

	return DeletedObjectPlaceholder;
}

UDeletedObjectPlaceholder* UDeletedObjectPlaceholder::RemoveFromPackage(UPackage* InPackage)
{
	if (IsRunningCommandlet())
	{
		return nullptr;
	}

	if (UDeletedObjectPlaceholder* DeletedObjectPlaceholder = InPackage ? UDeletedObjectPlaceholder::FindInPackage(InPackage) : nullptr)
	{
		DeletedObjectPlaceholder->Modify(false);
		DeletedObjectPlaceholder->ClearFlags(RF_Standalone);
		const FName NewName = MakeUniqueObjectName(nullptr, UDeletedObjectPlaceholder::StaticClass(), FName(*FString::Printf(TEXT("%s_Trashed"), *DeletedObjectPlaceholder->GetName())));
		DeletedObjectPlaceholder->Rename(*NewName.ToString(), GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty);
		DeletedObjectPlaceholder->SetExternalPackage(nullptr);
		return DeletedObjectPlaceholder;
	}
	return nullptr;
}

UDeletedObjectPlaceholder* UDeletedObjectPlaceholder::FindInPackage(const UPackage* InPackage)
{
	if (IsRunningCommandlet())
	{
		return nullptr;
	}

	UDeletedObjectPlaceholder* DeletedObjectPlaceholder = nullptr;
	ForEachObjectWithPackage(InPackage, [&DeletedObjectPlaceholder](UObject* Object)
	{
		DeletedObjectPlaceholder = Cast<UDeletedObjectPlaceholder>(Object);
		return !DeletedObjectPlaceholder;
	}, EGetObjectsFlags::None);
	return DeletedObjectPlaceholder;
}

void UDeletedObjectPlaceholder::PostEditUndo()
{
	Super::PostEditUndo();

	if (OriginalObject.IsValid() && (GetPackage() != GetTransientPackage()))
	{
		OnObjectCreated.Broadcast(this);
	}
}

#endif
