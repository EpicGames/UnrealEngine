// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UDemoNetDriver.cpp: Simulated network driver for recording and playing back game sessions.
=============================================================================*/

#include "Engine/DemoNetDriver.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/DemoNetConnection.h"
#include "Engine/GameInstance.h"
#include "Engine/LevelStreaming.h"
#include "UObject/Package.h"
#include "Engine/NetConnection.h"
#include "GameFramework/PlayerStart.h"
#include "Engine/LocalPlayer.h"
#include "Engine/PendingNetGame.h"
#include "EngineUtils.h"
#include "Engine/DemoPendingNetGame.h"
#include "Engine/ReplicationDriver.h"
#include "Net/DataReplication.h"
#include "Engine/ActorChannel.h"
#include "Engine/NetworkObjectList.h"
#include "GameFramework/SpectatorPawn.h"
#include "GameFramework/SpectatorPawnMovement.h"
#include "Math/Interval.h"
#include "Net/UnrealNetwork.h"
#include "UnrealEngine.h"
#include "Net/NetworkProfiler.h"
#include "GameFramework/GameStateBase.h"
#include "Misc/CommandLine.h"
#include "Stats/StatsMisc.h"
#include "Engine/ChildConnection.h"
#include "Net/ReplayPlaylistTracker.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DemoNetDriver)

DEFINE_LOG_CATEGORY( LogDemo );

#define DEMO_CSV_PROFILING_HELPERS_ENABLED (CSV_PROFILER && (!UE_BUILD_SHIPPING))

#if UE_BUILD_SHIPPING
CSV_DEFINE_CATEGORY(Demo, false);
#else
CSV_DEFINE_CATEGORY(Demo, true);
#endif

TAutoConsoleVariable<float> CVarDemoRecordHz( TEXT( "demo.RecordHz" ), 8, TEXT( "Maximum number of demo frames recorded per second" ) );
TAutoConsoleVariable<float> CVarDemoMinRecordHz(TEXT("demo.MinRecordHz"), 0, TEXT("Minimum number of demo frames recorded per second (use with care)"));
static TAutoConsoleVariable<float> CVarDemoTimeDilation( TEXT( "demo.TimeDilation" ), -1.0f, TEXT( "Override time dilation during demo playback (-1 = don't override)" ) );
static TAutoConsoleVariable<float> CVarDemoSkipTime( TEXT( "demo.SkipTime" ), 0, TEXT( "Skip fixed amount of network replay time (in seconds)" ) );
TAutoConsoleVariable<int32> CVarEnableCheckpoints( TEXT( "demo.EnableCheckpoints" ), 1, TEXT( "Whether or not checkpoints save on the server" ) );
static TAutoConsoleVariable<float> CVarGotoTimeInSeconds( TEXT( "demo.GotoTimeInSeconds" ), -1, TEXT( "For testing only, jump to a particular time" ) );
static TAutoConsoleVariable<int32> CVarDemoFastForwardDestroyTearOffActors( TEXT( "demo.FastForwardDestroyTearOffActors" ), 1, TEXT( "If true, the driver will destroy any torn-off actors immediately while fast-forwarding a replay." ) );
static TAutoConsoleVariable<int32> CVarDemoFastForwardSkipRepNotifies( TEXT( "demo.FastForwardSkipRepNotifies" ), 1, TEXT( "If true, the driver will optimize fast-forwarding by deferring calls to RepNotify functions until the fast-forward is complete. " ) );
static TAutoConsoleVariable<int32> CVarDemoQueueCheckpointChannels( TEXT( "demo.QueueCheckpointChannels" ), 1, TEXT( "If true, the driver will put all channels created during checkpoint loading into queuing mode, to amortize the cost of spawning new actors across multiple frames." ) );
static TAutoConsoleVariable<int32> CVarUseAdaptiveReplayUpdateFrequency( TEXT( "demo.UseAdaptiveReplayUpdateFrequency" ), 1, TEXT( "If 1, NetUpdateFrequency will be calculated based on how often actors actually write something when recording to a replay" ) );
static TAutoConsoleVariable<int32> CVarDemoAsyncLoadWorld( TEXT( "demo.AsyncLoadWorld" ), 0, TEXT( "If 1, we will use seamless server travel to load the replay world asynchronously" ) );
TAutoConsoleVariable<float> CVarCheckpointUploadDelayInSeconds( TEXT( "demo.CheckpointUploadDelayInSeconds" ), 30.0f, TEXT( "" ) );
static TAutoConsoleVariable<int32> CVarDemoLoadCheckpointGarbageCollect( TEXT( "demo.LoadCheckpointGarbageCollect" ), 1, TEXT("If nonzero, CollectGarbage will be called during LoadCheckpoint after the old actors and connection are cleaned up." ) );
TAutoConsoleVariable<float> CVarCheckpointSaveMaxMSPerFrameOverride( TEXT( "demo.CheckpointSaveMaxMSPerFrameOverride" ), -1.0f, TEXT( "If >= 0, this value will override the CheckpointSaveMaxMSPerFrame member variable, which is the maximum time allowed each frame to spend on saving a checkpoint. If 0, it will save the checkpoint in a single frame, regardless of how long it takes." ) );
TAutoConsoleVariable<int32> CVarDemoClientRecordAsyncEndOfFrame( TEXT( "demo.ClientRecordAsyncEndOfFrame" ), 0, TEXT( "If true, TickFlush will be called on a thread in parallel with Slate." ) );
static TAutoConsoleVariable<int32> CVarForceDisableAsyncPackageMapLoading( TEXT( "demo.ForceDisableAsyncPackageMapLoading" ), 0, TEXT( "If true, async package map loading of network assets will be disabled." ) );
TAutoConsoleVariable<int32> CVarDemoUseNetRelevancy( TEXT( "demo.UseNetRelevancy" ), 0, TEXT( "If 1, will enable relevancy checks and distance culling, using all connected clients as reference." ) );
static TAutoConsoleVariable<float> CVarDemoCullDistanceOverride( TEXT( "demo.CullDistanceOverride" ), 0.0f, TEXT( "If > 0, will represent distance from any viewer where actors will stop being recorded." ) );
static TAutoConsoleVariable<float> CVarDemoRecordHzWhenNotRelevant( TEXT( "demo.RecordHzWhenNotRelevant" ), 2.0f, TEXT( "Record at this frequency when actor is not relevant." ) );
static TAutoConsoleVariable<int32> CVarLoopDemo(TEXT("demo.Loop"), 0, TEXT("<1> : play replay from beginning once it reaches the end / <0> : stop replay at the end"));
static TAutoConsoleVariable<int32> CVarDemoFastForwardIgnoreRPCs( TEXT( "demo.FastForwardIgnoreRPCs" ), 1, TEXT( "If true, RPCs will be discarded during playback fast forward." ) );
static TAutoConsoleVariable<int32> CVarDemoLateActorDormancyCheck(TEXT("demo.LateActorDormancyCheck"), 1, TEXT("If true, check if an actor should become dormant as late as possible- when serializing it to the demo archive."));

static TAutoConsoleVariable<int32> CVarDemoJumpToEndOfLiveReplay(TEXT("demo.JumpToEndOfLiveReplay"), 1, TEXT("If true, fast forward to a few seconds before the end when starting playback, if the replay is still being recorded."));
static TAutoConsoleVariable<int32> CVarDemoInternalPauseChannels(TEXT("demo.InternalPauseChannels"), 1, TEXT("If true, run standard logic for PauseChannels rather than letting the game handle it via FOnPauseChannelsDelegate."));

static int32 GDemoLoopCount = 0;
static FAutoConsoleVariableRef CVarDemoLoopCount( TEXT( "demo.LoopCount" ), GDemoLoopCount, TEXT( "If > 1, will play the replay that many times before stopping." ) );

static int32 GDemoSaveRollbackActorState = 1;
static FAutoConsoleVariableRef CVarDemoSaveRollbackActorState( TEXT( "demo.SaveRollbackActorState" ), GDemoSaveRollbackActorState, TEXT( "If true, rollback actors will save some replicated state to apply when respawned." ) );

TAutoConsoleVariable<int32> CVarWithLevelStreamingFixes(TEXT("demo.WithLevelStreamingFixes"), 0, TEXT("If 1, provides fixes for level streaming (but breaks backwards compatibility)."));
TAutoConsoleVariable<int32> CVarWithDemoTimeBurnIn(TEXT("demo.WithTimeBurnIn"), 0, TEXT("If true, adds an on screen message with the current DemoTime and Changelist."));
TAutoConsoleVariable<int32> CVarWithDeltaCheckpoints(TEXT("demo.WithDeltaCheckpoints"), 0, TEXT("If true, record checkpoints as a delta from the previous checkpoint."));
TAutoConsoleVariable<int32> CVarWithGameSpecificFrameData(TEXT("demo.WithGameSpecificFrameData"), 0, TEXT("If true, allow game specific data to be recorded with each demo frame."));

static TAutoConsoleVariable<float> CVarDemoIncreaseRepPrioritizeThreshold(TEXT("demo.IncreaseRepPrioritizeThreshold"), 0.9f, TEXT("The % of Replicated to Prioritized actors at which prioritize time will be decreased."));
static TAutoConsoleVariable<float> CVarDemoDecreaseRepPrioritizeThreshold(TEXT("demo.DecreaseRepPrioritizeThreshold"), 0.7f, TEXT("The % of Replicated to Prioritized actors at which prioritize time will be increased."));
static TAutoConsoleVariable<float> CVarDemoMinimumRepPrioritizeTime(TEXT("demo.MinimumRepPrioritizePercent"), 0.3f, TEXT("Minimum percent of time that must be spent prioritizing actors, regardless of throttling."));
static TAutoConsoleVariable<float> CVarDemoMaximumRepPrioritizeTime(TEXT("demo.MaximumRepPrioritizePercent"), 0.7f, TEXT("Maximum percent of time that may be spent prioritizing actors, regardless of throttling."));

static TAutoConsoleVariable<int32> CVarFastForwardLevelsPausePlayback(TEXT("demo.FastForwardLevelsPausePlayback"), 0, TEXT("If true, pause channels and playback while fast forward levels task is running."));

namespace ReplayTaskNames
{
	static FName SkipTimeInSecondsTask("SkipTimeInSecondsTask");
	static FName JumpToLiveReplayTask("JumpToLiveReplayTask");
	static FName GotoTimeInSecondsTask("GotoTimeInSecondsTask");
	static FName FastForwardLevelsTask("FastForwardLevelsTask");
};

namespace UE::Net
{
	extern bool bNetReplicateOnlyBeginPlay;
}

// This is only intended for testing purposes
// A "better" way might be to throw together a GameplayDebuggerComponent or Category, so we could populate
// more than just the DemoTime.
static void ConditionallyDisplayBurnInTime(uint32 RecordedCL, float CurrentDemoTime)
{
	if (CVarWithDemoTimeBurnIn.GetValueOnAnyThread() != 0)
	{
		GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.f, FColor::Red, FString::Printf(TEXT("Current CL: %u | Recorded CL: %u | Time: %f"), FEngineVersion::Current().GetChangelist(), RecordedCL, CurrentDemoTime), true, FVector2D(3.f, 3.f));
	}
}

static bool ShouldActorGoDormantForDemo(const AActor* Actor, const UActorChannel* Channel)
{
	if ( Actor->NetDormancy <= DORM_Awake || !Channel || Channel->bPendingDormancy || Channel->Dormant )
{
		// Either shouldn't go dormant, or is already dormant
		return false;
	}

	return true;
}

namespace DemoNetDriverRecordingPrivate
{
	static float WarningTimeInterval = 60.f;
	static FAutoConsoleVariableRef CVarExceededBudgetWarningInterval(
		TEXT("Demo.ExceededBudgetWarningInterval"),
		WarningTimeInterval,
		TEXT("When > 0, we will wait this many seconds between logging warnings for demo recording exceeding time budgets.")
	);	

	static bool RecordUnicastRPCs = false;
	static FAutoConsoleVariableRef CVarRecordUnicastRPCs(
		TEXT("demo.RecordUnicastRPCs"),
		RecordUnicastRPCs,
		TEXT("When true, also record unicast client rpcs on actors that share a net driver name with the demo driver.")
	);

	static TAutoConsoleVariable<int32> CVarDemoForcePersistentLevelPriority(TEXT("demo.ForcePersistentLevelPriority"), 0, TEXT("If true, force persistent level to record first when prioritizing and using streaming level fixes."));
	static TAutoConsoleVariable<int32> CVarDemoDestructionInfoPriority(TEXT("demo.DestructionInfoPriority"), MAX_int32, TEXT("Replay net priority assigned to destruction infos during recording."));
	static TAutoConsoleVariable<int32> CVarDemoLateDestructionInfoPrioritize(TEXT("demo.LateDestructionInfoPrioritize"), 0, TEXT("If true, process destruction infos at the end of the prioritization phase."));
	static TAutoConsoleVariable<float> CVarDemoViewTargetPriorityScale(TEXT("demo.ViewTargetPriorityScale"), 3.0, TEXT("Scale view target priority by this value when prioritization is enabled."));
	static TAutoConsoleVariable<float> CVarDemoMaximumRecDestructionInfoTime(TEXT("demo.MaximumRecDestructionInfoTime"), 0.2f, TEXT("Maximum percentage of frame to use replicating destruction infos, if per frame limit is enabled."));
	
	static FAutoConsoleCommandWithWorldAndArgs DemoMaxDesiredRecordTimeMS(
		TEXT("Demo.MaxDesiredRecordTimeMS"),
		TEXT("Set max desired record time in MS on demo driver of the current world."),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
			[](const TArray<FString>& Params, UWorld* World)
			{
				if (World)
				{
					if (UDemoNetDriver* Driver = World->GetDemoNetDriver())
					{
						if (Params.Num() > 0)
						{
							const float TimeInMS = FCString::Atof(*Params[0]);

							Driver->SetMaxDesiredRecordTimeMS(TimeInMS);
						}
					}
				}
			}));

	static FAutoConsoleCommandWithWorldAndArgs DemoCheckpointSaveMaxMSPerFrame(
		TEXT("Demo.CheckpointSaveMaxMSPerFrame"),
		TEXT("Set max checkpoint record time in MS on demo driver of the current world."),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
			[](const TArray<FString>& Params, UWorld* World)
			{
				if (World)
				{
					if (UDemoNetDriver* Driver = World->GetDemoNetDriver())
					{
						if (Params.Num() > 0)
						{
							const float TimeInMS = FCString::Atof(*Params[0]);

							Driver->SetCheckpointSaveMaxMSPerFrame(TimeInMS);
						}
					}
				}
			}));

	static FAutoConsoleCommandWithWorldAndArgs DemoActorPrioritizationEnabled(
		TEXT("Demo.ActorPrioritizationEnabled"),
		TEXT("Set whether or not actor prioritization is enabled on demo driver of the current world."),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
			[](const TArray<FString>& Params, UWorld* World)
			{
				if (World)
				{
					if (UDemoNetDriver* Driver = World->GetDemoNetDriver())
					{
						if (Params.Num() > 0)
						{
							const bool bPrioritize = FCString::ToBool(*Params[0]);

							Driver->SetActorPrioritizationEnabled(bPrioritize);
						}
					}
				}
			}));

	static FAutoConsoleCommandWithWorldAndArgs DemoSetLocalViewerOverride(
		TEXT("Demo.SetLocalViewerOverride"),
		TEXT("Set first local player controller as the viewer override on demo driver of the current world."),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
			[](const TArray<FString>& Params, UWorld* World)
			{
				if (World)
				{
					if (UDemoNetDriver* Driver = World->GetDemoNetDriver())
					{
						if (APlayerController* ViewerPC = GEngine->GetFirstLocalPlayerController(World))
						{
							Driver->SetViewerOverride(ViewerPC);
						}
					}
				}
			}));

#if !UE_BUILD_SHIPPING
	static FAutoConsoleCommandWithWorldAndArgs DemoTestWriteEvent(
		TEXT("Demo.TestWriteEvent"),
		TEXT("Add or update a test replay event on the currently recording replay, with an optional argument for event size in bytes"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
			[](const TArray<FString>& Params, UWorld* World)
			{
				if (World)
				{
					if (UDemoNetDriver* Driver = World->GetDemoNetDriver())
					{
						if (APlayerController* ViewerPC = GEngine->GetFirstLocalPlayerController(World))
						{
							int32 EventSize = 32;

							if (Params.Num() > 0)
							{
								EventSize = FCString::Atoi(*Params[0]);
							}

							TArray<uint8> EventData;
							EventData.AddZeroed(EventSize);

							Driver->AddOrUpdateEvent(TEXT("Test"), TEXT("Test"), TEXT("Test"), EventData);
						}
					}
				}
			}));
#endif
}

struct FDemoBudgetLogHelper
{
	enum EBudgetCategory
	{
		Prioritization,
		Replication,
		CATEGORY_COUNT
	};

	FDemoBudgetLogHelper(FString&& Identifier)
		: Identifier(MoveTemp(Identifier))
	{
		ResetCounters();
	}

	void NewFrame()
	{
		if (FirstWarningTime != 0.f)
		{
			++NumFrames;
			bOverBudgetThisFrame = false;

			const double Time = FPlatformTime::Seconds();
			if (Time - FirstWarningTime > DemoNetDriverRecordingPrivate::WarningTimeInterval)
			{
				if (UE_LOG_ACTIVE(LogDemo, Log))
				{
					TArray<FString> LogLines;
					LogLines.Reserve((CATEGORY_COUNT * 2) + 1);

					LogLines.Emplace(FString::Printf(TEXT("%s: Recorded Frames: %d, Frames Over Budget: %d"), *Identifier, NumFrames, NumFramesOverBudget));

					for (int32 i = 0; i < CATEGORY_COUNT; ++i)
					{
						LogLines.Emplace(FString::Printf(TEXT("Total number of over budget frames in category %d: %d"), i, NumFramesOverBudgetByCategory[i]));

						if (NumFramesOverBudgetByCategory[i] > 0)
						{
							LogLines.Emplace(MoveTemp(LogSamplesByBudget[i]));
						}
					}

					UE_LOGF(LogDemo, Log, "%ls", *FString::Join(LogLines, TEXT("\n")));
				}

				ResetCounters();
			}
		}
	}

	template<typename... T>
	void MarkFrameOverBudget(EBudgetCategory Category, ::UE::Core::TCheckedFormatString<FString::FmtCharType, T...> Format, T... Args)
	{
		if (!UE_LOG_ACTIVE(LogDemo, Log))
		{
			return;
		}

		if (DemoNetDriverRecordingPrivate::WarningTimeInterval == 0.f)
		{
			UE_LOGF(LogDemo, Log, "%ls", *FString::Printf(Format, Args...));
			return;
		}

		if (!bOverBudgetThisFrame)
		{
			bOverBudgetThisFrame = true;
			++NumFramesOverBudget;

			if (FirstWarningTime == 0.f)
			{
				FirstWarningTime = FPlatformTime::Seconds();
			}
		}

		++NumFramesOverBudgetByCategory[Category];
		if (LogSamplesByBudget[Category].IsEmpty())
		{
			LogSamplesByBudget[Category] = FString::Printf(Format, Args...);
		}
	}

	void ResetCounters()
	{
		NumFrames = 0;
		NumFramesOverBudget = 0;
		FirstWarningTime = 0.f;
		for (int32 i = 0; i < CATEGORY_COUNT; ++i)
		{
			NumFramesOverBudgetByCategory[i] = 0;
			LogSamplesByBudget[i] = FString();
		}
	}

private:

	bool bOverBudgetThisFrame = false;

	int32 NumFrames;
	int32 NumFramesOverBudget;

	double FirstWarningTime = 0.f;

	int32 NumFramesOverBudgetByCategory[CATEGORY_COUNT];
	FString LogSamplesByBudget[CATEGORY_COUNT];

	FString Identifier;
};

class FPendingTaskHelper
{
// TODO: Consider making these private, and adding explicit friend access for the tasks that need them.
public:

	static bool LoadCheckpoint(UDemoNetDriver* DemoNetDriver, const FGotoResult& GotoResult)
	{
		return DemoNetDriver->LoadCheckpoint(GotoResult);
	}

	static bool FastForwardLevels(UDemoNetDriver* DemoNetDriver, const FGotoResult& GotoResult)
	{
		return DemoNetDriver->FastForwardLevels(GotoResult);
	}

	static float GetLastProcessedPacketTime(UDemoNetDriver* DemoNetDriver)
	{
		return DemoNetDriver->LastProcessedPacketTime;
	}
};

class FScopedAllowExistingChannelIndex
{
public:
	FScopedAllowExistingChannelIndex(FScopedAllowExistingChannelIndex&&) = delete;
	FScopedAllowExistingChannelIndex(const FScopedAllowExistingChannelIndex&) = delete;
	FScopedAllowExistingChannelIndex& operator=(const FScopedAllowExistingChannelIndex&) = delete;
	FScopedAllowExistingChannelIndex& operator=(FScopedAllowExistingChannelIndex&&) = delete;

	FScopedAllowExistingChannelIndex(UNetConnection* InConnection):
		Connection(InConnection)
	{
		if (Connection.IsValid())
		{
			Connection->SetAllowExistingChannelIndex(true);
		}	
	}

	~FScopedAllowExistingChannelIndex()
	{
		if (Connection.IsValid())
		{
			Connection->SetAllowExistingChannelIndex(false);
		}	
	}

private:
	TWeakObjectPtr<UNetConnection> Connection;
};

class FJumpToLiveReplayTask : public FQueuedReplayTask
{
public:
	FJumpToLiveReplayTask(UDemoNetDriver* InDriver) : FQueuedReplayTask(InDriver)
	{
		if (Driver.IsValid())
		{
			InitialTotalDemoTime	= Driver->GetDemoTotalTime();
			TaskStartTime			= FPlatformTime::Seconds();
		}
	}

	virtual void StartTask()
	{
	}

	virtual bool Tick() override
	{
		if (!Driver.IsValid())
		{
			return true;
		}

		if (!Driver->GetReplayStreamer()->IsLive())
		{
			// The replay is no longer live, so don't try to jump to end
			return true;
		}

		// Wait for the most recent live time
		const bool bHasNewReplayTime = (Driver->GetDemoTotalTime() != InitialTotalDemoTime);

		// If we haven't gotten a new time from the demo by now, assume it might not be live, and just jump to the end now so we don't hang forever
		const bool bTimeExpired = (FPlatformTime::Seconds() - TaskStartTime >= 15.0);

		if (bHasNewReplayTime || bTimeExpired)
		{
			if (bTimeExpired)
			{
				UE_LOGF(LogDemo, Warning, "FJumpToLiveReplayTask::Tick: Too much time since last live update.");
			}

			// We're ready to jump to the end now
			Driver->JumpToEndOfLiveReplay();
			return true;
		}

		// Waiting to get the latest update
		return false;
	}

	virtual FName GetName() const override
	{
		return ReplayTaskNames::JumpToLiveReplayTask;
	}

private:
	float	InitialTotalDemoTime;		// Initial total demo time. This is used to wait until we get a more updated time so we jump to the most recent end time
	double	TaskStartTime;				// This is the time the task started. If too much real-time passes, we'll just jump to the current end
};

class FGotoTimeInSecondsTask : public FQueuedReplayTask
{
public:
	FGotoTimeInSecondsTask(UDemoNetDriver* InDriver, const float InTimeInSeconds) : FQueuedReplayTask(InDriver), TimeInSeconds(InTimeInSeconds)
	{
	}

	virtual void StartTask() override
	{		
		if (!Driver.IsValid())
		{
			return;
		}

		check(!GotoResult.IsSet());
		check(!Driver->IsFastForwarding());

		OldTimeInSeconds = Driver->GetDemoCurrentTime();	// Rember current time, so we can restore on failure
		Driver->SetDemoCurrentTime(TimeInSeconds);	// Also, update current time so HUD reflects desired scrub time now

		// Clamp time
		Driver->SetDemoCurrentTime(FMath::Clamp(Driver->GetDemoCurrentTime(), 0.0f, Driver->GetDemoTotalTime() - 0.01f));

		EReplayCheckpointType CheckpointType = Driver->HasDeltaCheckpoints() ? EReplayCheckpointType::Delta : EReplayCheckpointType::Full;

		// Tell the streamer to start going to this time
		Driver->GetReplayStreamer()->GotoTimeInMS(Driver->GetDemoCurrentTimeInMS(), FGotoCallback::CreateSP(this, &FGotoTimeInSecondsTask::CheckpointReady), CheckpointType);

		// Pause channels while we wait (so the world is paused while we wait for the new stream location to load)
		Driver->PauseChannels(true);
	}

	virtual bool Tick() override
	{
		if (!Driver.IsValid())
		{
			// Detect failure case
			return true;
		}
		else if (GotoResult.IsSet())
		{
			if (!GotoResult->WasSuccessful())
			{
				return true;
			}
			else if (GotoResult->ExtraTimeMS > 0 && !Driver->GetReplayStreamer()->IsDataAvailable())
			{
				// Wait for rest of stream before loading checkpoint
				// We do this so we can load the checkpoint and fastforward the stream all at once
				// We do this so that the OnReps don't stay queued up outside of this frame
				return false;
			}

			// We're done
			return FPendingTaskHelper::LoadCheckpoint(Driver.Get(), GotoResult.GetValue());
		}

		return false;
	}

	virtual FName GetName() const override
	{
		return ReplayTaskNames::GotoTimeInSecondsTask;
	}

	void CheckpointReady(const FGotoResult& Result)
	{
		check(!GotoResult.IsSet());
		GotoResult = Result;

		if (!Driver.IsValid())
	{
			return;
		}

		if (!Result.WasSuccessful())
		{
			UE_LOGF(LogDemo, Warning, "FGotoTimeInSecondsTask::CheckpointReady: Failed to go to checkpoint.");

			// Restore old demo time
			Driver->SetDemoCurrentTime(OldTimeInSeconds);

			// Call delegate if any
			Driver->NotifyGotoTimeFinished(false);
		}
	}

	// So we can restore on failure
	float OldTimeInSeconds;		
	float TimeInSeconds;
	TOptional<FGotoResult> GotoResult;
};

class FSkipTimeInSecondsTask : public FQueuedReplayTask
{
public:
	FSkipTimeInSecondsTask(UDemoNetDriver* InDriver, const float InSecondsToSkip) : FQueuedReplayTask(InDriver), SecondsToSkip(InSecondsToSkip)
	{
	}

	virtual void StartTask() override
	{
		if (!Driver.IsValid())
		{
			return;
		}

		check(!Driver->IsFastForwarding());

		const uint32 TimeInMSToCheck = FMath::Clamp(Driver->GetDemoCurrentTimeInMS() + (uint32)(SecondsToSkip * 1000), (uint32)0, Driver->GetReplayStreamer()->GetTotalDemoTime());

		Driver->GetReplayStreamer()->SetHighPriorityTimeRange(Driver->GetDemoCurrentTimeInMS(), TimeInMSToCheck);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Driver->SkipTimeInternal(SecondsToSkip, true, false);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual bool Tick() override
	{
		// The real work was done in StartTask, so we're done
		return true;
	}

	virtual FName GetName() const override
	{
		return ReplayTaskNames::SkipTimeInSecondsTask;
	}

	float SecondsToSkip;
};

class FFastForwardLevelsTask : public FQueuedReplayTask
{
public:

	FFastForwardLevelsTask( UDemoNetDriver* InDriver ) : FQueuedReplayTask( InDriver ), GotoTime(0), bSkipWork(false)
	{
	}

	virtual void StartTask() override
	{
		if (!Driver.IsValid())
		{
			return;
		}

		check(!Driver->IsFastForwarding());

		// If there's a GotoTimeInSeconds task pending, we don't need to do any work.
		// That task should trigger a full checkpoint load.
		// Only check the next task, to avoid issues with SkipTime / JumpToLive not having updated levels.
		if (Driver->GetNextQueuedTaskName() == ReplayTaskNames::GotoTimeInSecondsTask)
		{
			bSkipWork = true;
		}
		else
		{
			// Make sure we request all the data we need so we don't end up doing a "partial" fast forward which
			// could cause the level to miss network updates.
			const float LastProcessedPacketTime = FPendingTaskHelper::GetLastProcessedPacketTime(Driver.Get());
			GotoTime = LastProcessedPacketTime * 1000;

			EReplayCheckpointType CheckpointType = Driver->HasDeltaCheckpoints() ? EReplayCheckpointType::Delta : EReplayCheckpointType::Full;

			Driver->GetReplayStreamer()->GotoTimeInMS(GotoTime, FGotoCallback::CreateSP(this, &FFastForwardLevelsTask::CheckpointReady), CheckpointType);

			if (CVarFastForwardLevelsPausePlayback.GetValueOnAnyThread() != 0)
			{
				// Pause channels while we wait (so the world is paused while we wait for the new stream location to load)
				Driver->PauseChannels(true);
			}
		}
	}

	virtual bool Tick() override
	{
		if (bSkipWork)
		{
			return true;
		}
		else if (!Driver.IsValid())
		{
			return true;
		}
		else if (GotoResult.IsSet())
		{
			// if this task is not pausing the rest of the replay stream, make sure there is data available for the current time or we could miss packets
			const float LastProcessedPacketTime = FPendingTaskHelper::GetLastProcessedPacketTime(Driver.Get());
			const uint32 AvailableDataEndTime = (CVarFastForwardLevelsPausePlayback.GetValueOnAnyThread() != 0) ? GotoTime : LastProcessedPacketTime * 1000;

			if (!GotoResult->WasSuccessful())
			{
				return true;
			}
		
			// If not all data is available, we could end only partially fast forwarding the levels.
			// Note, IsDataAvailable may return false even if IsDataAvailableForTimeRange is true.
			// So, check both to ensure that we don't end up skipping data in FastForwardLevels.
			else if (GotoResult->ExtraTimeMS > 0 && !(Driver->GetReplayStreamer()->IsDataAvailable() && Driver->GetReplayStreamer()->IsDataAvailableForTimeRange(GotoTime - GotoResult->ExtraTimeMS, AvailableDataEndTime)))
			{
				return false;
			}

			return FPendingTaskHelper::FastForwardLevels(Driver.Get(), GotoResult.GetValue());
		}

		return false;
	}

	virtual FName GetName() const override
	{
		return ReplayTaskNames::FastForwardLevelsTask;
	}

	virtual bool ShouldPausePlayback() const override
	{
		return (CVarFastForwardLevelsPausePlayback.GetValueOnAnyThread() != 0);
	}

	void CheckpointReady(const FGotoResult& Result)
	{
		check(!GotoResult.IsSet());

		GotoResult = Result;

		if (!Result.WasSuccessful())
		{
			UE_LOGF(LogDemo, Warning, "FFastForwardLevelsTask::CheckpointReady: Failed to get checkpoint.");
		}
	}

private:

	uint32 GotoTime;
	bool bSkipWork;
	TOptional<FGotoResult> GotoResult;
};

/*-----------------------------------------------------------------------------
	UDemoNetDriver.
-----------------------------------------------------------------------------*/

void UDemoNetDriver::InitDefaults()
{
	DemoSessionID = FGuid::NewGuid().ToString().ToLower();
	SetCurrentLevelIndex(0);
	bIsWaitingForHeaderDownload = false;
	bIsWaitingForStream = false;
	MaxArchiveReadPos = 0;
	bNeverApplyNetworkEmulationSettings = true;
	bSkipServerReplicateActors = true;
	bSkipClearVoicePackets = true;
	bSkipStartupActorRollback = false;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		LevelIntervals.Reserve(512);

		ReplayHelper.PlaybackDemoHeader.SetDefaultNetworkVersions();
	}

	RecordBuildConsiderAndPrioritizeTimeSlice = CVarDemoMaximumRepPrioritizeTime.GetValueOnGameThread();
	RecordDestructionInfoReplicationTimeSlice = DemoNetDriverRecordingPrivate::CVarDemoMaximumRecDestructionInfoTime.GetValueOnAnyThread();
}

UDemoNetDriver::UDemoNetDriver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InitDefaults();
}

UDemoNetDriver::UDemoNetDriver(FVTableHelper& Helper)
	: Super(Helper)
{
	InitDefaults();
}

UDemoNetDriver::~UDemoNetDriver()
{
}

void UDemoNetDriver::AddReplayTask(FQueuedReplayTask* NewTask)
{
	UE_LOGF(LogDemo, Verbose, "UDemoNetDriver::AddReplayTask. Name: %ls", *NewTask->GetName().ToString());

	QueuedReplayTasks.Emplace(NewTask);

	// Give this task a chance to immediately start if nothing else is happening
	if (!IsAnyTaskPending())
	{
		ProcessReplayTasks();	
	}
}

bool UDemoNetDriver::IsAnyTaskPending() const
{
	return (QueuedReplayTasks.Num() > 0) || ActiveReplayTask.IsValid();
}

void UDemoNetDriver::ClearReplayTasks()
{
	QueuedReplayTasks.Empty();

	ActiveReplayTask = nullptr;
}

bool UDemoNetDriver::ProcessReplayTasks()
{
	// Store a shared pointer to the current task in a local variable so that if
	// the task itself causes tasks to be cleared (for example, if it calls StopDemo()
	// in StartTask() or Tick()), the current task won't be destroyed immediately.
	TSharedPtr<FQueuedReplayTask> LocalActiveTask;

	if (!ActiveReplayTask.IsValid() && QueuedReplayTasks.Num() > 0)
	{
		// If we don't have an active task, pull one off now
		ActiveReplayTask = QueuedReplayTasks[0];
		LocalActiveTask = ActiveReplayTask;
		QueuedReplayTasks.RemoveAt(0);

		UE_LOGF(LogDemo, Verbose, "UDemoNetDriver::ProcessReplayTasks. Name: %ls", *ActiveReplayTask->GetName().ToString());

		// Start the task
		LocalActiveTask->StartTask();
	}

	// Tick the currently active task
	if (ActiveReplayTask.IsValid())
	{
		LocalActiveTask = ActiveReplayTask;

		if (!LocalActiveTask->Tick())
		{
			// Task isn't done, we can return
			return !LocalActiveTask->ShouldPausePlayback();
		}

		// This task is now done
		ActiveReplayTask = nullptr;
	}

	return true;	// No tasks to process
}

bool UDemoNetDriver::IsNamedTaskInQueue(const FName& Name) const
{
	if (ActiveReplayTask.IsValid() && ActiveReplayTask->GetName() == Name)
	{
		return true;
	}

	for (const TSharedRef<FQueuedReplayTask>& Task : QueuedReplayTasks)
	{
		if (Task->GetName() == Name)
		{
			return true;
		}
	}

	return false;
}

FName UDemoNetDriver::GetNextQueuedTaskName() const
{
	return QueuedReplayTasks.Num() > 0 ? QueuedReplayTasks[0]->GetName() : NAME_None;
}

bool UDemoNetDriver::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	if (Super::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error))
	{
		bChannelsArePaused = false;
		ResetElapsedTime();
		bIsFastForwarding				= false;
		bIsFastForwardingForCheckpoint	= false;
		bIsRestoringStartupActors		= false;
		bWasStartStreamingSuccessful	= true;
		SavedReplicatedWorldTimeSeconds	= 0.0;
		SavedSecondsToSkip				= 0.0f;
		MaxDesiredRecordTimeMS			= -1.0f;
		ViewerOverride					= nullptr;
		bPrioritizeActors				= false;
		PlaybackPacketIndex				= 0;
		CheckpointSaveMaxMSPerFrame		= -1.0f;

		if (FParse::Param(FCommandLine::Get(), TEXT("skipreplayrollback")))
		{
			bSkipStartupActorRollback = true;
		}

		RecordBuildConsiderAndPrioritizeTimeSlice = CVarDemoMaximumRepPrioritizeTime.GetValueOnAnyThread();
		RecordDestructionInfoReplicationTimeSlice = DemoNetDriverRecordingPrivate::CVarDemoMaximumRecDestructionInfoTime.GetValueOnAnyThread();

		if (RelevantTimeout == 0.0f)
		{
			RelevantTimeout = 5.0f;
		}

		ResetDemoState();

		if (URL.HasOption(TEXT("MaxDesiredReplayRecordTimeMS")))
		{
			MaxDesiredRecordTimeMS = FCString::Atof(URL.GetOption(TEXT("MaxDesiredReplayRecordTimeMS="), nullptr));
		}

		if (URL.HasOption(TEXT("CheckpointSaveMaxMSPerFrame")))
		{
			CheckpointSaveMaxMSPerFrame = FCString::Atof(URL.GetOption(TEXT("CheckpointSaveMaxMSPerFrame="), nullptr));
		}

		if (URL.HasOption(TEXT("ActorPrioritizationEnabled")))
		{
			bPrioritizeActors = FCString::Atoi(URL.GetOption(TEXT("ActorPrioritizationEnabled="), nullptr)) != 0;
		}

		ReplayStreamer = ReplayHelper.Init(URL);

		ReplayHelper.SetAnalyticsProvider(AnalyticsProvider);
		ReplayHelper.CheckpointSaveMaxMSPerFrame = CheckpointSaveMaxMSPerFrame;

		// if the helper encounters an error, stop the presses
		ReplayHelper.OnReplayRecordError.BindUObject(this, &UDemoNetDriver::NotifyDemoRecordFailure);
		ReplayHelper.OnReplayPlaybackError.BindUObject(this, &UDemoNetDriver::NotifyDemoPlaybackError);

		return true;
	}

	return false;
}

void UDemoNetDriver::FinishDestroy()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Make sure we stop any recording/playing that might be going on
		if (IsRecording() || IsPlaying())
		{
			StopDemo();
		}
	}

	CleanUpSplitscreenConnections(true);
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

	ReplayHelper.OnReplayRecordError.Unbind();
	ReplayHelper.OnReplayPlaybackError.Unbind();
	
	if (World)
	{
		World->RemoveOnActorDestroyedHandler(DelegateHandleActorPreDestroy);
	}

	Super::FinishDestroy();
}

FString UDemoNetDriver::LowLevelGetNetworkNumber()
{
	return FString();
}

void UDemoNetDriver::ResetDemoState()
{
	SetDemoCurrentTime(0.0f);
	SetDemoTotalTime(0.0f);
	LastProcessedPacketTime	= 0.0f;
	PlaybackPacketIndex		= 0;

	bIsFastForwarding = false;
	bIsFastForwardingForCheckpoint = false;
	bIsRestoringStartupActors = false;
	bWasStartStreamingSuccessful = false;
	bIsWaitingForHeaderDownload = false;
	bIsWaitingForStream = false;
	bIsFinalizingFastForward = false;

	PlaybackPackets.Empty();

	ReplayHelper.ResetState();
}

bool UDemoNetDriver::InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error)
{
	ReplayHelper.bRecording = false;

	// handle default initialization
	if (!World || !World->GetGameInstance() || !InitBase(true, InNotify, ConnectURL, false, Error))
	{
		ReplayHelper.NotifyReplayError(EReplayResult::InitConnect);
		return false;
	}

	GetNetGuidCache()->SetNetworkChecksumMode(FNetGUIDCache::ENetworkChecksumMode::SaveButIgnore);

	if (CVarForceDisableAsyncPackageMapLoading.GetValueOnGameThread() > 0)
	{
		GetNetGuidCache()->SetAsyncLoadMode(FNetGUIDCache::EAsyncLoadMode::ForceDisable);
	}
	else
	{
		GetNetGuidCache()->SetAsyncLoadMode(FNetGUIDCache::EAsyncLoadMode::UseCVar);
	}

	// Playback, local machine is a client, and the demo stream acts "as if" it's the server.
	ServerConnection = NewObject<UNetConnection>(GetTransientPackage(), UDemoNetConnection::StaticClass());
	ServerConnection->InitConnection(this, USOCK_Pending, ConnectURL, 1000000);

	const TCHAR* const LevelPrefixOverrideOption = ConnectURL.GetOption(TEXT("LevelPrefixOverride="), nullptr);
	if (LevelPrefixOverrideOption)
	{
		SetDuplicateLevelID(FCString::Atoi(LevelPrefixOverrideOption));
	}

	if (GetDuplicateLevelID() == -1)
	{
		// Set this driver as the demo net driver for the source level collection.
		FLevelCollection* const SourceCollection = World->FindCollectionByType(ELevelCollectionType::DynamicSourceLevels);
		if (SourceCollection)
		{
			SourceCollection->SetDemoNetDriver(this);
		}
	}
	else
	{
		// Set this driver as the demo net driver for the duplicate level collection.
		FLevelCollection* const DuplicateCollection = World->FindCollectionByType(ELevelCollectionType::DynamicDuplicatedLevels);
		if (DuplicateCollection)
		{
			DuplicateCollection->SetDemoNetDriver(this);
		}
	}

	bIsWaitingForStream = true;
	bWasStartStreamingSuccessful = true;

	ReplayHelper.ActiveReplayName = ConnectURL.Map;

	TArray<int32> UserIndices;
	for (FLocalPlayerIterator LocalPlayerIt(GEngine, World); LocalPlayerIt; ++LocalPlayerIt)
	{
		if (*LocalPlayerIt)
		{
			UserIndices.Add(LocalPlayerIt->GetControllerId());
		}
	}
	
	FStartStreamingParameters Params;
	Params.CustomName = ConnectURL.Map;
	Params.DemoURL = GetDemoURL();
	Params.UserIndices = MoveTemp(UserIndices);
	Params.bRecord = false;
	Params.ReplayVersion = FNetworkVersion::GetReplayVersion();

	GetReplayStreamer()->StartStreaming(Params, FStartStreamingCallback::CreateUObject(this, &UDemoNetDriver::ReplayStreamingReady));

	return bWasStartStreamingSuccessful;
}

bool UDemoNetDriver::InitConnectInternal(FString& Error)
{
	ResetDemoState();

	if (!ReplayHelper.ReadPlaybackDemoHeader(Error))
	{
		return false;
	}

	// Set network version on connection
	ReplayHelper.SetPlaybackNetworkVersions(ServerConnection);

	// Create fake control channel
	CreateInitialClientChannels();
	
	// Default async world loading to the cvar value...
	bool bAsyncLoadWorld = CVarDemoAsyncLoadWorld.GetValueOnGameThread() > 0;

	// ...but allow it to be overridden via a command-line option.
	const TCHAR* const AsyncLoadWorldOverrideOption = ReplayHelper.DemoURL.GetOption(TEXT("AsyncLoadWorldOverride="), nullptr);
	if (AsyncLoadWorldOverrideOption)
	{
		bAsyncLoadWorld = FCString::ToBool(AsyncLoadWorldOverrideOption);
	}

	// Hook up to get notifications so we know when a travel is complete (LoadMap or Seamless).
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddUObject(this, &ThisClass::OnPostLoadMapWithWorld);

	if (GetDuplicateLevelID() == -1)
	{
		if (bAsyncLoadWorld && World->WorldType != EWorldType::PIE) // Editor doesn't support async map travel
		{
			ReplayHelper.LevelNamesAndTimes = ReplayHelper.PlaybackDemoHeader.LevelNamesAndTimes;

			// FIXME: Test for failure!!!
			ProcessSeamlessTravel(0);
		}
		else
		{
			// Bypass UDemoPendingNetLevel
			FString LoadMapError;

			FURL LocalDemoURL;
			LocalDemoURL.Map = ReplayHelper.PlaybackDemoHeader.LevelNamesAndTimes[0].LevelName;

			if (!GEngine->MakeSureMapNameIsValid(LocalDemoURL.Map))
			{
				ReplayHelper.NotifyReplayError(EReplayResult::LoadMap);
				return false;
			}

			FWorldContext * WorldContext = GEngine->GetWorldContextFromWorld(World);

			if (WorldContext == nullptr)
			{
				Error = FString::Printf(TEXT("No world context"));
				UE_LOGF(LogDemo, Error, "UDemoNetDriver::InitConnect: %ls", *Error);
				ReplayHelper.NotifyReplayError(EReplayResult::InitConnect);
				return false;
			}

			World->ClearDemoNetDriver();
			SetWorld(nullptr);

			auto NewPendingNetGame = NewObject<UDemoPendingNetGame>();

			// Set up the pending net game so that the engine can call LoadMap on the next tick.
			NewPendingNetGame->SetDemoNetDriver(this);
			NewPendingNetGame->URL = LocalDemoURL;
			NewPendingNetGame->bSuccessfullyConnected = true;

			WorldContext->PendingNetGame = NewPendingNetGame;
		}
	}
	else
	{
		ReplayHelper.ResetLevelStatuses();
		ReplayHelper.ResetLevelMap();
	}

	return true;
}

bool UDemoNetDriver::InitListen(FNetworkNotify* InNotify, FURL& ListenURL, bool bReuseAddressAndPort, FString& Error)
{
	ReplayHelper.bRecording = true;

	if (!InitBase(false, InNotify, ListenURL, bReuseAddressAndPort, Error))
	{
		return false;
	}

	check(World != nullptr);

	AWorldSettings* WorldSettings = World->GetWorldSettings(); 

	if (!WorldSettings)
	{
		Error = TEXT("No WorldSettings!!");
		return false;
	}

	// Recording, local machine is server, demo stream acts "as if" it's a client.
	UDemoNetConnection* Connection = NewObject<UDemoNetConnection>();
	Connection->InitConnection(this, USOCK_Open, ListenURL, 1000000);

	AddClientConnection(Connection);

	Connection->SetClientWorldPackageName(World->GetOutermost()->GetFName());

	// Technically, NetDriver's can be renamed so this could become stale.
	// However, it's only used for logging and DemoNetDriver's are typically given a special name.
	BudgetLogHelper = MakeUnique<FDemoBudgetLogHelper>(NetDriverName.ToString());

	ReplayHelper.StartRecording(Connection);

	// Spawn the demo recording spectator.
	SpawnDemoRecSpectator(Connection, ListenURL);

	return true;
}

void UDemoNetDriver::NotifyStreamingLevelUnload( ULevel* InLevel )
{
	if (InLevel && !InLevel->bClientOnlyVisible && HasLevelStreamingFixes() && IsPlaying())
	{
		const FName FilterLevelName = InLevel->GetOutermost()->GetFName();

		// We can't just iterate over the levels actors, because the ones in the queue will already have been destroyed.
		for (TMap<FString, FRollbackNetStartupActorInfo>::TIterator RollbackIt = RollbackNetStartupActors.CreateIterator(); RollbackIt; ++RollbackIt)
		{
			if (RollbackIt.Value().LevelName == FilterLevelName)
			{
				RollbackIt.RemoveCurrent();
			}
		}
	}

	Super::NotifyStreamingLevelUnload(InLevel);
}

void UDemoNetDriver::OnPostLoadMapWithWorld(UWorld* InWorld)
{
	if (InWorld != nullptr && InWorld == World)
	{
		if (HasLevelStreamingFixes())
		{
			if (IsPlaying())
			{
				ReplayHelper.ResetLevelStatuses();
			}
			else
			{
				ReplayHelper.ClearLevelStreamingState();
			}
		}

		if (IsPlaying())
		{
			ReplayHelper.ResetLevelMap();
		}
		else
		{
			ReplayHelper.ClearLevelMap();
		}
	}
}

void UDemoNetDriver::DiffActorProperties(UActorChannel* const ActorChannel)
{
	if (!ActorChannel || !ActorChannel->GetActor())
	{
		return;
	}

	auto DiffObjectProperties = [](const FObjectReplicator& ObjectReplicator)
	{
		if (const UObject* const ReplicatedObject = ObjectReplicator.GetObject())
		{
			FReceivingRepState* const ReceivingRepState = ObjectReplicator.RepState->GetReceivingRepState();
			const FRepShadowDataBuffer ShadowData(ReceivingRepState->StaticBuffer.GetData());
			const FConstRepObjectDataBuffer RepObjectData(ReplicatedObject);

			ObjectReplicator.RepLayout->DiffProperties(&(ReceivingRepState->RepNotifies), ShadowData, RepObjectData, EDiffPropertiesFlags::Sync);
		}
	};

	// Make sure we diff Actor first
	const FObjectReplicator& ActorReplicator = ActorChannel->GetActorReplicationData();
	DiffObjectProperties(ActorReplicator);

	// Diff any Components and SubObjects
	for (const auto& ReplicatorPair : ActorChannel->ReplicationMap)
	{
		const FObjectReplicator& ObjectReplicator = ReplicatorPair.Value.Get();
		// We don't need to diff Actor again
		if (ActorReplicator.GetObject() == ObjectReplicator.GetObject())
		{
			continue;
		}
		DiffObjectProperties(ObjectReplicator);
	}
}

bool UDemoNetDriver::ContinueListen(FURL& ListenURL)
{
	if (IsRecording() && ensure(IsRecordingPaused()))
	{
		SetCurrentLevelIndex(GetCurrentLevelIndex() + 1);

		PauseRecording(false);

		// Delete the old player controller, we're going to create a new one (and we can't leave this one hanging around)
		if (SpectatorController != nullptr)
		{
			SpectatorController->Player = nullptr;		// Force APlayerController::DestroyNetworkActorHandled to return false
			World->DestroyActor(SpectatorController, true);
			SpectatorControllers.Empty();
			SpectatorController = nullptr;
		}

		SpawnDemoRecSpectator(ClientConnections[0], ListenURL);

		// Force a checkpoint to be created in the next tick - We need a checkpoint right after traveling so that scrubbing
		// from a different level will have essentially an "empty" checkpoint to work from.
		SetLastCheckpointTime(-1 * CVarCheckpointUploadDelayInSeconds.GetValueOnGameThread());
		return true;
	}
	return false;
}

bool UDemoNetDriver::IsRecording() const
{
	return ClientConnections.Num() > 0 && ClientConnections[0] != nullptr && ClientConnections[0]->GetConnectionState() != USOCK_Closed;
}

bool UDemoNetDriver::IsPlaying() const
{
	// ServerConnection may be deleted / recreated during checkpoint loading.
	return IsLoadingCheckpoint() || (ServerConnection != nullptr && ServerConnection->GetConnectionState() != USOCK_Closed);
}

bool UDemoNetDriver::IsServer() const
{
	return (ServerConnection == nullptr) || IsRecording();
}

bool UDemoNetDriver::ShouldTickFlushAsyncEndOfFrame() const
{
	return GEngine && GEngine->ShouldDoAsyncEndOfFrameTasks() && CVarDemoClientRecordAsyncEndOfFrame.GetValueOnAnyThread() != 0 && World && World->IsRecordingClientReplay();
}

void UDemoNetDriver::TickFlush(float DeltaSeconds)
{
	if (!ShouldTickFlushAsyncEndOfFrame())
	{
		TickFlushInternal(DeltaSeconds);
	}
}

void UDemoNetDriver::PostTickFlush()
{
	Super::PostTickFlush();

	// process this outside of tick in case we want to destroy the driver in response
	if (PendingRecordFailure.IsSet())
	{
		if (World)
		{
			if (UGameInstance* GameInstance = World->GetGameInstance())
			{
				GameInstance->HandleDemoRecordFailure(PendingRecordFailure.GetValue().GetResult());
			}
		}

		PendingRecordFailure.Reset();
	}
}

void UDemoNetDriver::TickFlushAsyncEndOfFrame(float DeltaSeconds)
{
	if (ShouldTickFlushAsyncEndOfFrame())
	{
		TickFlushInternal(DeltaSeconds);
		SetIsInTick(false); //PostTickFlush isn't called after the async TickFlush, so set bInTick to false here
	}
}

/** Accounts for the network time we spent in the demo driver. */
double GTickFlushDemoDriverTimeSeconds = 0.0;

void UDemoNetDriver::TickFlushInternal(float DeltaSeconds)
{
	LLM_SCOPE(ELLMTag::Replays);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(DemoRecording);

	GTickFlushDemoDriverTimeSeconds = 0.0;
	FSimpleScopeSecondsCounter ScopedTimer(GTickFlushDemoDriverTimeSeconds);

	// Set the context on the world for this driver's level collection.
	const int32 FoundCollectionIndex = World ? World->GetLevelCollections().IndexOfByPredicate([this](const FLevelCollection& Collection)
	{
		return Collection.GetDemoNetDriver() == this;
	}) : INDEX_NONE;

	FScopedLevelCollectionContextSwitch LCSwitch(FoundCollectionIndex, World);

	Super::TickFlush(DeltaSeconds);

	if (!IsRecording() || bIsWaitingForStream)
	{
		// Nothing to do
		return;
	}

	TSharedPtr<INetworkReplayStreamer> Streamer = GetReplayStreamer();

	if (UE::Net::EHandleNetResult::Handled == Streamer->HandleLastError(ReplayHelper.ResultManager))
	{
		ReplayHelper.NotifyReplayError(EReplayResult::StreamerError);
		return;
	}

	if (IsRecordingPaused())
	{
		return;
	}

	FArchive* FileAr = Streamer->GetStreamingArchive();

	if (FileAr == nullptr)
	{
		UE_LOGF(LogDemo, Error, "UDemoNetDriver::TickFlush: FileAr == nullptr");
		ReplayHelper.NotifyReplayError(EReplayResult::MissingArchive);
		return;
	}

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Net replay record time"), STAT_ReplayRecordTime, STATGROUP_Net);

	const double StartTime = FPlatformTime::Seconds();

	TickDemoRecord(DeltaSeconds);

	const double EndTime = FPlatformTime::Seconds();

	const double RecordTotalTime = (EndTime - StartTime);

	// While recording, the CurrentCL is the same as the recording CL.
	ConditionallyDisplayBurnInTime(FEngineVersion::Current().GetChangelist(), GetDemoCurrentTime());

	MaxRecordTime = FMath::Max(MaxRecordTime, RecordTotalTime);

	AccumulatedRecordTime += RecordTotalTime;

	RecordCountSinceFlush++;

	const double DemoElapsedTime = EndTime - LastRecordAvgFlush;

	const double AVG_FLUSH_TIME_IN_SECONDS = 2;

	if (DemoElapsedTime > AVG_FLUSH_TIME_IN_SECONDS && RecordCountSinceFlush > 0)
	{
		const float AvgTimeMS = (AccumulatedRecordTime / RecordCountSinceFlush) * 1000;
		const float MaxRecordTimeMS = MaxRecordTime * 1000;

		if (AvgTimeMS > 8.0f)//|| MaxRecordTimeMS > 6.0f )
		{
			UE_LOGF(LogDemo, Verbose, "UDemoNetDriver::TickFlush: SLOW FRAME. Avg: %2.2f, Max: %2.2f, Actors: %i", AvgTimeMS, MaxRecordTimeMS, GetNetworkObjectList().GetActiveObjects().Num());
		}

		LastRecordAvgFlush		= EndTime;
		AccumulatedRecordTime	= 0;
		MaxRecordTime			= 0;
		RecordCountSinceFlush	= 0;
	}
}

void UDemoNetDriver::TickDispatch(float DeltaSeconds)
{
	LLM_SCOPE(ELLMTag::Replays);

	// Set the context on the world for this driver's level collection.
	const int32 FoundCollectionIndex = World ? World->GetLevelCollections().IndexOfByPredicate([this](const FLevelCollection& Collection)
	{
		return Collection.GetDemoNetDriver() == this;
	}) : INDEX_NONE;

	FScopedLevelCollectionContextSwitch LCSwitch(FoundCollectionIndex, World);

	Super::TickDispatch(DeltaSeconds);

	if (!IsPlaying() || bIsWaitingForStream)
	{
		// Nothing to do
		return;
	}

	if (UE::Net::EHandleNetResult::Handled == GetReplayStreamer()->HandleLastError(ReplayHelper.ResultManager))
	{
		ReplayHelper.NotifyReplayError(EReplayResult::StreamerError);
		return;
	}

	FArchive* FileAr = GetReplayStreamer()->GetStreamingArchive();
	if (FileAr == nullptr)
	{
		UE_LOGF(LogDemo, Error, "UDemoNetDriver::TickDispatch: FileAr == nullptr");
		ReplayHelper.NotifyReplayError(EReplayResult::MissingArchive);
		return;
	}

	if (!HasLevelStreamingFixes())
	{
		// Wait until all levels are streamed in
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			if (StreamingLevel && StreamingLevel->ShouldBeLoaded() && (!StreamingLevel->IsLevelLoaded() || !StreamingLevel->GetLoadedLevel()->GetOutermost()->IsFullyLoaded() || !StreamingLevel->IsLevelVisible()))
			{
				// Abort, we have more streaming levels to load
				return;
			}
		}
	}	

	if (CVarDemoTimeDilation.GetValueOnGameThread() >= 0.0f)
	{
		World->GetWorldSettings()->DemoPlayTimeDilation = CVarDemoTimeDilation.GetValueOnGameThread();
	}

	// DeltaSeconds that is padded in, is unclampped and not time dilated
	DeltaSeconds = FReplayHelper::GetClampedDeltaSeconds( World, DeltaSeconds );

	// Update time dilation on spectator pawn to compensate for any demo dilation 
	//	(we want to continue to fly around in real-time)
	for (APlayerController* CurSpectatorController : SpectatorControllers)
	{
		if (CurSpectatorController == nullptr)
		{
			continue;
		}

		if ( World->GetWorldSettings()->DemoPlayTimeDilation > UE_KINDA_SMALL_NUMBER )
		{
			CurSpectatorController->CustomTimeDilation = 1.0f / World->GetWorldSettings()->DemoPlayTimeDilation;
		}
		else
		{
			CurSpectatorController->CustomTimeDilation = 1.0f;
		}

		if (CurSpectatorController->GetSpectatorPawn() != nullptr)
		{
			CurSpectatorController->GetSpectatorPawn()->CustomTimeDilation = CurSpectatorController->CustomTimeDilation;

			CurSpectatorController->GetSpectatorPawn()->PrimaryActorTick.bTickEvenWhenPaused = true;

			USpectatorPawnMovement* SpectatorMovement = Cast<USpectatorPawnMovement>(CurSpectatorController->GetSpectatorPawn()->GetMovementComponent());

			if ( SpectatorMovement )
			{
				//SpectatorMovement->bIgnoreTimeDilation = true;
				SpectatorMovement->PrimaryComponentTick.bTickEvenWhenPaused = true;
			}
		}
	}

	TickDemoPlayback(DeltaSeconds);

	// Used LastProcessedPacketTime because it will correlate better with recorded frame time.
	ConditionallyDisplayBurnInTime(ReplayHelper.PlaybackDemoHeader.EngineVersion.GetChangelist(), LastProcessedPacketTime);
}

void UDemoNetDriver::ProcessRemoteFunction(AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, UObject* SubObject)
{
#if !UE_BUILD_SHIPPING
	bool bBlockSendRPC = false;

	SendRPCDel.ExecuteIfBound(Actor, Function, Parameters, OutParms, Stack, SubObject, bBlockSendRPC);

	if (!bBlockSendRPC)
#endif
	{
		if (IsRecording())
		{
			const bool bRecordRPC = DemoNetDriverRecordingPrivate::RecordUnicastRPCs ? ShouldReplicateFunction(Actor, Function) : EnumHasAnyFlags(Function->FunctionFlags, FUNC_NetMulticast);

			if (bRecordRPC)
			{
				const bool bIsRelevant = !Actor->bOnlyRelevantToOwner || (Actor->GetNetDriverName() == NetDriverName);

				if (bIsRelevant)
				{
					// Handle role swapping if this is a client-recorded replay.
					FScopedActorRoleSwap RoleSwap(Actor);

					InternalProcessRemoteFunction(Actor, SubObject, ClientConnections[0], Function, Parameters, OutParms, Stack, IsServer());
				}
			}
		}
	}
}

bool UDemoNetDriver::ShouldClientDestroyTearOffActors() const
{
	if (CVarDemoFastForwardDestroyTearOffActors.GetValueOnGameThread() != 0)
	{
		return bIsFastForwarding;
	}

	return false;
}

bool UDemoNetDriver::ShouldSkipRepNotifies() const
{
	if (CVarDemoFastForwardSkipRepNotifies.GetValueOnAnyThread() != 0)
	{
		return bIsFastForwarding;
	}

	return false;
}

void UDemoNetDriver::StopDemo()
{
	if (!IsRecording() && !IsPlaying())
	{
		UE_LOGF(LogDemo, Log, "StopDemo: No demo is playing");
		ClearReplayTasks();
		ReplayHelper.ActiveReplayName.Empty();
		ResetDemoState();
		return;
	}

	UE_LOGF(LogDemo, Log, "StopDemo: Demo %ls stopped at frame %d", *ReplayHelper.DemoURL.Map, GetDemoFrameNum());

	if (!ServerConnection)
	{
		// let GC cleanup the object
		if (ClientConnections.Num() > 0 && ClientConnections[0] != nullptr)
		{
			ClientConnections[0]->Close();
		}
	}
	else
	{
		// flush out any pending network traffic
		ServerConnection->FlushNet();

		ServerConnection->SetConnectionState(USOCK_Closed);
		ServerConnection->Close();
	}

	ReplayHelper.StopReplay();

	SpectatorControllers.Empty();
	SpectatorController = nullptr;

	ClearReplayTasks();
	ResetDemoState();

	check(!IsRecording() && !IsPlaying());
}

/*-----------------------------------------------------------------------------
Demo Recording tick.
-----------------------------------------------------------------------------*/

bool UDemoNetDriver::DemoReplicateActor(AActor* Actor, UNetConnection* Connection, bool bMustReplicate)
{
	return ReplayHelper.ReplicateActor(Actor, Connection, bMustReplicate);
}

void UDemoNetDriver::AddEvent(const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	AddOrUpdateEvent(FString(), Group, Meta, Data);
}

void UDemoNetDriver::AddOrUpdateEvent(const FString& Name, const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	ReplayHelper.AddOrUpdateEvent(Name, Group, Meta, Data);
}

void UDemoNetDriver::EnumerateEvents(const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->EnumerateEvents(Group, Delegate);
	}
}

void UDemoNetDriver::RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->RequestEventData(EventID, Delegate);
	}
}

void UDemoNetDriver::EnumerateEventsForActiveReplay(const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->EnumerateEvents(GetActiveReplayName(), Group, Delegate);
	}
}

void UDemoNetDriver::EnumerateEventsForActiveReplay(const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->EnumerateEvents(GetActiveReplayName(), Group, UserIndex, Delegate);
	}
}

void UDemoNetDriver::RequestEventDataForActiveReplay(const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->RequestEventData(GetActiveReplayName(), EventID, Delegate);
	}
}

void UDemoNetDriver::RequestEventDataForActiveReplay(const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->RequestEventData(GetActiveReplayName(), EventID, UserIndex, Delegate);
	}
}

void UDemoNetDriver::RequestEventGroupDataForActiveReplay(const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->RequestEventGroupData(GetActiveReplayName(), Group, Delegate);
	}
}

void UDemoNetDriver::RequestEventGroupDataForActiveReplay(const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->RequestEventGroupData(GetActiveReplayName(), Group, UserIndex, Delegate);
	}
}

/**
* FReplayViewer
* Used when demo.UseNetRelevancy enabled
* Tracks all of the possible viewers of a replay that we use to determine relevancy
*/
class FReplayViewer
{
public:
	FReplayViewer(const UNetConnection* Connection) :
		Viewer(Connection->PlayerController ? Connection->PlayerController : Connection->OwningActor), 
		ViewTarget(Connection->PlayerController ? Connection->PlayerController->GetViewTarget() : ToRawPtr(Connection->OwningActor))
	{
		Location = ViewTarget ? ViewTarget->GetActorLocation() : FVector::ZeroVector;
	}

	AActor*		Viewer;
	AActor*		ViewTarget;
	FVector		Location;
};

class FRepActorsParams
{
public:
	FRepActorsParams(FRepActorsParams&&) = delete;
	FRepActorsParams(const FRepActorsParams&) = delete;
	FRepActorsParams& operator=(const FRepActorsParams&) = delete;
	FRepActorsParams& operator=(FRepActorsParams&&) = delete;

	FRepActorsParams(UDemoNetConnection* InConnection, const bool bInUseAdaptiveNetFrequency, const bool bInDoFindActorChannel, const bool bInDoCheckDormancy,
					const float InMinRecordHz, const float InMaxRecordHz, const float InServerTickTime,
					const double InReplicationStartTimeSeconds, const double InTimeLimitSeconds, const double InDestructionInfoTimeLimitSeconds):
		Connection(InConnection),
		bUseAdapativeNetFrequency(bInUseAdaptiveNetFrequency),
		bDoFindActorChannel(bInDoFindActorChannel),
		bDoCheckDormancy(bInDoCheckDormancy),
		NumActorsReplicated(0),
		NumDestructionInfosReplicated(0),
		MinRecordHz(InMinRecordHz),
		MaxRecordHz(InMaxRecordHz),
		ServerTickTime(InServerTickTime),
		ReplicationStartTimeSeconds(InReplicationStartTimeSeconds),
		TimeLimitSeconds(InTimeLimitSeconds),
		DestructionInfoTimeLimitSeconds(InDestructionInfoTimeLimitSeconds),
		TotalDestructionInfoRecordTime(0.0)
	{
	}

	UDemoNetConnection* Connection;
	const bool bUseAdapativeNetFrequency;
	const bool bDoFindActorChannel;
	const bool bDoCheckDormancy;
	int32 NumActorsReplicated;
	int32 NumDestructionInfosReplicated;
	const float MinRecordHz;
	const float MaxRecordHz;
	const float ServerTickTime;
	const double ReplicationStartTimeSeconds;
	const double TimeLimitSeconds;
	const double DestructionInfoTimeLimitSeconds;
	double TotalDestructionInfoRecordTime;
};

void UDemoNetDriver::TickDemoRecord(float DeltaSeconds)
{
	if (!IsRecording() || IsRecordingPaused())
	{
		return;
	}

	CSV_SCOPED_TIMING_STAT(Demo, DemoRecordTime);

	// DeltaSeconds that is padded in, is unclamped and not time dilated
	SetDemoCurrentTime(GetDemoCurrentTime() + FReplayHelper::GetClampedDeltaSeconds(World, DeltaSeconds));

	ReplayHelper.ReplayStreamer->UpdateTotalDemoTime(GetDemoCurrentTimeInMS());

	if (ReplayHelper.GetCheckpointSaveState() != FReplayHelper::ECheckpointSaveState::Idle)
	{
		// If we're in the middle of saving a checkpoint, then update that now and return
		ReplayHelper.TickCheckpoint(ClientConnections[0]);
		return;
	}
	else
	{
		TickDemoRecordFrame(DeltaSeconds);

		// Save a checkpoint if it's time
		if (CVarEnableCheckpoints.GetValueOnAnyThread() == 1)
		{
			check(ReplayHelper.GetCheckpointSaveState() == FReplayHelper::ECheckpointSaveState::Idle);		// We early out above, so this shouldn't be possible

			if (ReplayHelper.ShouldSaveCheckpoint())
			{
				ReplayHelper.SaveCheckpoint(ClientConnections[0]);
			}
		}
	}
}

void UDemoNetDriver::BuildSortedLevelPriorityOnLevels(const TArray<FDemoActorPriority>& PrioritizedActorList, TArray<FLevelnterval>& OutLevelIntervals)
{
	OutLevelIntervals.Reset();

	// Find level intervals
	const int32 Count = PrioritizedActorList.Num();
	const FDemoActorPriority* Priorities = PrioritizedActorList.GetData();

	const bool bHighPriorityPersistentLevel = DemoNetDriverRecordingPrivate::CVarDemoForcePersistentLevelPriority.GetValueOnAnyThread() != 0;

	for (int32 Index = 0; Index < Count;)
	{
		const UObject* CurrentLevel = Priorities[Index].Level;

		FLevelnterval Interval;
		Interval.StartIndex = Index;

		if (bHighPriorityPersistentLevel && World && (CurrentLevel == World->PersistentLevel))
		{
			Interval.Priority = MAX_int32;
		}
		else
		{
			Interval.Priority = Priorities[Index].ActorPriority.Priority;
		}

		Interval.LevelIndex = (CurrentLevel != nullptr ? ReplayHelper.FindOrAddLevelStatus(*Cast<ULevel>(CurrentLevel)).LevelIndex + 1 : 0);

		while (Index < Count && Priorities[Index].Level == CurrentLevel)
		{
			++Index;
		}

		Interval.Count = Index - Interval.StartIndex;

		OutLevelIntervals.Add(Interval);
	}

	// Sort intervals on priority
	OutLevelIntervals.Sort([](const FLevelnterval& A, const FLevelnterval& B) { return (B.Priority < A.Priority) || ((A.Priority == B.Priority) && (A.LevelIndex < B.LevelIndex)); });
}

void UDemoNetDriver::TickDemoRecordFrame(float DeltaSeconds)
{
	FArchive* FileAr = GetReplayStreamer()->GetStreamingArchive();

	if (FileAr == nullptr)
	{
		return;
	}

	const double RecordFrameStartTime = FPlatformTime::Seconds();
	const double RecordTimeLimit = (MaxDesiredRecordTimeMS / 1000.f);

	// Mark any new streaming levels, so that they are saved out this frame
	if (!HasLevelStreamingFixes())
	{
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			if (StreamingLevel == nullptr || !StreamingLevel->ShouldBeLoaded() || StreamingLevel->ShouldBeAlwaysLoaded())
			{
				continue;
			}

			TWeakObjectPtr<UObject> WeakStreamingLevel;
			WeakStreamingLevel = StreamingLevel;
			if (!ReplayHelper.UniqueStreamingLevels.Contains(WeakStreamingLevel))
			{
				ReplayHelper.UniqueStreamingLevels.Add(WeakStreamingLevel);
				ReplayHelper.NewStreamingLevelsThisFrame.Add(WeakStreamingLevel);
			}
		}
	}

	// Save out a frame
	ReplayHelper.DemoFrameNum++;

	ReplicationFrame++;
	BudgetLogHelper->NewFrame();

	UDemoNetConnection* ClientConnection = CastChecked<UDemoNetConnection>(ClientConnections[0]);

	// flush out any pending network traffic
	FReplayHelper::FlushNetChecked(*ClientConnection);

	float ServerTickTime = GEngine->GetMaxTickRate( DeltaSeconds );
	if (ServerTickTime == 0.0)
	{
		ServerTickTime = DeltaSeconds;
	}
	else
	{
		ServerTickTime	= 1.0 / ServerTickTime;
	}

	// Build priority list
	FNetworkObjectList& NetObjectList = GetNetworkObjectList();
	const FNetworkObjectList::FNetworkObjectSet& ActiveObjectSet = NetObjectList.GetActiveObjects();
	const int32 NumActiveObjects = ActiveObjectSet.Num();

	PrioritizedActors.Reset(NumActiveObjects);

	// Set the location of the connection's viewtarget for prioritization.
	FVector ViewLocation = FVector::ZeroVector;
	FVector ViewDirection = FVector::ZeroVector;
	APlayerController* CachedViewerOverride = ViewerOverride.Get();
	APlayerController* Viewer = CachedViewerOverride ? CachedViewerOverride : ClientConnection->GetPlayerController(World);
	AActor* ViewTarget = Viewer ? Viewer->GetViewTarget() : nullptr;
	
	if (ViewTarget)
	{
		ViewLocation = ViewTarget->GetActorLocation();
		ViewDirection = ViewTarget->GetActorRotation().Vector();
	}

	const bool bDoCheckDormancyEarly = CVarDemoLateActorDormancyCheck.GetValueOnAnyThread() == 0;
	const bool bLateDestructionInfos = DemoNetDriverRecordingPrivate::CVarDemoLateDestructionInfoPrioritize.GetValueOnAnyThread() != 0;
	const bool bDoPrioritizeActors = bPrioritizeActors;
	const bool bDoFindActorChannelEarly = bDoPrioritizeActors || bDoCheckDormancyEarly;

	int32 ActorsPrioritized = 0;
	int32 DestructionInfosPrioritized = 0;

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Replay prioritize time"), STAT_ReplayPrioritizeTime, STATGROUP_Net);

		const double ConsiderTimeLimit = RecordTimeLimit * RecordBuildConsiderAndPrioritizeTimeSlice;
		auto HasConsiderTimeBeenExhausted = [ConsiderTimeLimit, RecordFrameStartTime, RecordTimeLimit]() -> bool
		{
			return RecordTimeLimit > 0.f && ((FPlatformTime::Seconds() - RecordFrameStartTime) > ConsiderTimeLimit);
		};

		auto PrioritizeDestructionInfos = [ClientConnection, this, &HasConsiderTimeBeenExhausted]()
		{
			SCOPED_NAMED_EVENT(UDemoNetDriver_PrioritizeDestroyedOrDormantActors, FColor::Green);

			// Add destroyed actors that the client may not have a channel for
			FDemoActorPriority DestroyedActorPriority;
			DestroyedActorPriority.ActorPriority.Priority = DemoNetDriverRecordingPrivate::CVarDemoDestructionInfoPriority.GetValueOnAnyThread();

			for (auto DestroyedOrDormantGUID = ClientConnection->GetDestroyedStartupOrDormantActorGUIDs().CreateIterator(); DestroyedOrDormantGUID; ++DestroyedOrDormantGUID)
			{
				TUniquePtr<FActorDestructionInfo>& DInfo = DestroyedStartupOrDormantActors.FindChecked(*DestroyedOrDormantGUID);
				DestroyedActorPriority.ActorPriority.DestructionInfo = DInfo.Get();
				DestroyedActorPriority.Level = HasLevelStreamingFixes() ? DestroyedActorPriority.ActorPriority.DestructionInfo->Level.Get() : nullptr;

				PrioritizedActors.Add(DestroyedActorPriority);

				if (HasConsiderTimeBeenExhausted())
				{
					UE_LOGF(LogDemo, Verbose, "Consider time exhaused prioritizing destruction infos.");
					break;
				}
			}
		};

		if (!bLateDestructionInfos)
		{
			PrioritizeDestructionInfos();

			DestructionInfosPrioritized = PrioritizedActors.Num();
		}

		if (!HasConsiderTimeBeenExhausted())
		{
			TArray< FReplayViewer, TInlineAllocator<16> > ReplayViewers;

			const bool bUseNetRelevancy = CVarDemoUseNetRelevancy.GetValueOnAnyThread() > 0 && World->NetDriver != nullptr && World->NetDriver->IsServer();

			// If we're using relevancy, consider all connections as possible viewing sources
			if (bUseNetRelevancy)
			{
				for (UNetConnection* Connection : World->NetDriver->ClientConnections)
				{
					FReplayViewer ReplayViewer(Connection);
					if (ReplayViewer.ViewTarget != nullptr)
					{
						ReplayViewers.Add(MoveTemp(ReplayViewer));
					}
				}
			}

			const float CullDistanceOverride = CVarDemoCullDistanceOverride.GetValueOnAnyThread();
			const float CullDistanceOverrideSq = CullDistanceOverride > 0.0f ? FMath::Square(CullDistanceOverride) : 0.0f;

			const float RecordHzWhenNotRelevant = CVarDemoRecordHzWhenNotRelevant.GetValueOnAnyThread();
			const float UpdateDelayWhenNotRelevant = RecordHzWhenNotRelevant > 0.0f ? 1.0f / RecordHzWhenNotRelevant : 0.5f;

			struct FActorRemoveInfo
			{
				AActor* Actor = nullptr;
				bool bIsValidActor = false;
			};
			TArray<FActorRemoveInfo, TInlineAllocator<128>> ActorsToRemove;

			FDemoActorPriority DemoActorPriority;
			FActorPriority& ActorPriority = DemoActorPriority.ActorPriority;

			const bool bDeltaCheckpoint = HasDeltaCheckpoints();

			const float CurrentTime = GetDemoCurrentTime();

			int32 ProcessedCount = 0;

			for (const TSharedPtr<FNetworkObjectInfo>& ObjectInfo : ActiveObjectSet)
			{
				FNetworkObjectInfo* ActorInfo = ObjectInfo.Get();

				++ProcessedCount;

				if (GetDemoCurrentTime() > ActorInfo->NextUpdateTime)
				{
					AActor* Actor = ActorInfo->Actor;

					if (!ActorInfo->WeakActor.IsValid())
					{
						ActorsToRemove.Add({.Actor = Actor, .bIsValidActor = false});
						continue;
					}

					// During client recording, a torn-off actor will already have its remote role set to None, but
					// we still need to replicate it one more time so that the recorded replay knows it's been torn-off as well.
					if (Actor->GetRemoteRole() == ROLE_None && !Actor->GetTearOff())
					{
						ActorsToRemove.Add({.Actor = Actor, .bIsValidActor = true});
						continue;
					}

					if (IsDormInitialStartupActor(Actor))
					{
						ActorsToRemove.Add({.Actor = Actor, .bIsValidActor = true});
						continue;
					}

					if (!Actor->bRelevantForNetworkReplays)
					{
						ActorsToRemove.Add({.Actor = Actor, .bIsValidActor = true});
						continue;
					}

					if (UE::Net::bNetReplicateOnlyBeginPlay && !Actor->HasActorBegunPlay())
					{
						UE_LOGF(LogNet, Verbose, "DemoNetDriver::ReplicateActor ignored the Actor since he was not BeginPlay yet: %ls", *GetPathNameSafe(Actor));
						continue;
					}

					// We check ActorInfo->LastNetUpdateTime < KINDA_SMALL_NUMBER to force at least one update for each actor
					const bool bWasRecentlyRelevant = (ActorInfo->LastNetUpdateTimestamp < UE_KINDA_SMALL_NUMBER) || ((GetElapsedTime() - ActorInfo->LastNetUpdateTimestamp) < RelevantTimeout);

					bool bIsRelevant = !bUseNetRelevancy || Actor->bAlwaysRelevant || Actor == ClientConnection->PlayerController || (ActorInfo->ForceRelevantFrame >= ReplicationFrame);

					if (!bIsRelevant)
					{
						// Assume this actor is relevant as long as *any* viewer says so
						for (const FReplayViewer& ReplayViewer : ReplayViewers)
						{
							if (Actor->IsReplayRelevantFor(ReplayViewer.Viewer, ReplayViewer.ViewTarget, ReplayViewer.Location, CullDistanceOverrideSq))
							{
								bIsRelevant = true;
								break;
							}
						}
					}

					if (!bIsRelevant && !bWasRecentlyRelevant)
					{
						// Actor is not relevant (or previously relevant), so skip and set next update time based on demo.RecordHzWhenNotRelevant
						ActorInfo->NextUpdateTime = CurrentTime + UpdateDelayWhenNotRelevant;
						continue;
					}

					UActorChannel* Channel = nullptr;
					if (bDoFindActorChannelEarly)
					{
						Channel = ClientConnection->FindActorChannelRef(ActorInfo->WeakActor);

						// Check dormancy
						if (bDoCheckDormancyEarly && Channel && ShouldActorGoDormantForDemo(Actor, Channel))
						{
							// Either shouldn't go dormant, or is already dormant
							Channel->StartBecomingDormant();
						}
					}

					ActorPriority.ActorInfo = ActorInfo;
					ActorPriority.Channel = Channel;
					DemoActorPriority.Level = Actor->GetOuter();

					if (bDoPrioritizeActors) // implies bDoFindActorChannelEarly is true
					{
						const double LastReplicationTime = Channel ? (GetElapsedTime() - Channel->LastUpdateTime) : SpawnPrioritySeconds;
						float ReplayPriority = 65536.0f * Actor->GetReplayPriority(ViewLocation, ViewDirection, Viewer, ViewTarget, Channel, LastReplicationTime);
						
						if (Actor == ViewTarget)
						{
							ReplayPriority = ReplayPriority * DemoNetDriverRecordingPrivate::CVarDemoViewTargetPriorityScale.GetValueOnAnyThread();
						}

						// clamp into a valid range prior to rounding to avoid potential undefined behavior
						ActorPriority.Priority = FMath::RoundToInt(FMath::Clamp(ReplayPriority, (float)(MIN_int32 + 10), (float)(MAX_int32 - 10)));
					}

					PrioritizedActors.Add(DemoActorPriority);

					ActorInfo->bDirtyForReplay = bDeltaCheckpoint;

					if (bIsRelevant)
					{
						ActorInfo->LastNetUpdateTimestamp = GetElapsedTime();
					}
				}

				if (HasConsiderTimeBeenExhausted())
				{
					UE_LOGF(LogDemo, Verbose, "Consider time exhaused while iterating the active object list [%d/%d]", ProcessedCount, ActiveObjectSet.Num());
					break;
				}
			}

			{
				SCOPED_NAMED_EVENT(UDemoNetDriver_PrioritizeRemoveActors, FColor::Green);

				// Always remove necessary actors, don't time slice this.
				for (FActorRemoveInfo& RemoveInfo : ActorsToRemove)
				{
					if (RemoveInfo.bIsValidActor)
					{
						RemoveNetworkActor(RemoveInfo.Actor);
					}
					else
					{
						RemoveInvalidNetworkActor(RemoveInfo.Actor);
					}
				}
			}
		}

		if (bLateDestructionInfos)
		{
			ActorsPrioritized = PrioritizedActors.Num();

			if (!HasConsiderTimeBeenExhausted())
			{
				PrioritizeDestructionInfos();
				DestructionInfosPrioritized = PrioritizedActors.Num();
			}
			else
			{
				UE_LOGF(LogDemo, Verbose, "Consider time exhausted without processing destruction infos");
			}
		}
		else
		{
			ActorsPrioritized = PrioritizedActors.Num() - DestructionInfosPrioritized;
		}
	}

	if (HasLevelStreamingFixes())
	{
		SCOPED_NAMED_EVENT(UDemoNetDriver_PrioritizeLevelSort, FColor::Green);
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Replay actor level sorting time."), STAT_ReplayLevelSorting, STATGROUP_Net);

		if (bPrioritizeActors)
		{
			UE_LOGF(LogDemo, Verbose, "bPrioritizeActors and HasLevelStreamingFixes are both enabled. This will undo some prioritization work.");
		}

		// Sort by Level and priority, If the order of levels are relevant we need a second pass on the array to find the intervals of the levels and sort those on "level with netobject with highest priority"
		// but since prioritization is disabled the order is arbitrary so there is really no use to do the extra work 
		PrioritizedActors.Sort([](const FDemoActorPriority& A, const FDemoActorPriority& B) { return (B.Level < A.Level) || ((B.Level == A.Level) && (B.ActorPriority.Priority < A.ActorPriority.Priority)); });

		// Find intervals in sorted priority lists with the same level and sort the intervals based on priority of first Object in each interval.
		// Intervals are then used to determine the order we write out the replicated objects as we write one packet per level.
		BuildSortedLevelPriorityOnLevels(PrioritizedActors, LevelIntervals);
	}
	else if (bPrioritizeActors)
	{
		// Sort on priority
		PrioritizedActors.Sort([](const FDemoActorPriority& A, const FDemoActorPriority& B) { return B.ActorPriority.Priority < A.ActorPriority.Priority; });
	}

	const double PrioritizeEndTime = FPlatformTime::Seconds();
	const double TotalPrioritizeActorsTime = (PrioritizeEndTime - RecordFrameStartTime);
	const float TotalPrioritizeActorsTimeMS = TotalPrioritizeActorsTime * 1000.f;

	CSV_CUSTOM_STAT(Demo, DemoPrioritizeTime, TotalPrioritizeActorsTimeMS, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Demo, DemoNumActiveObjects, NumActiveObjects, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Demo, DemoPrioritizedActors, ActorsPrioritized, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Demo, DemoPrioritizedDestInfos, DestructionInfosPrioritized, ECsvCustomStatOp::Set);

	const int32 NumPrioritizedActors = PrioritizedActors.Num();

	// Make sure we're under the desired recording time quota, if any.
	// See ReplicatePriorizeActor.
	if (RecordTimeLimit > 0.0f && TotalPrioritizeActorsTime > RecordTimeLimit)
	{
		BudgetLogHelper->MarkFrameOverBudget(
				FDemoBudgetLogHelper::Prioritization,
				TEXT("Exceeded maximum desired recording time (during Prioritization).  Max: %.3fms, TimeSpent: %.3fms, Active Actors: %d, Prioritized Actors: %d"),
				MaxDesiredRecordTimeMS, TotalPrioritizeActorsTimeMS, NumActiveObjects, NumPrioritizedActors);
	}

	float MinRecordHz = CVarDemoMinRecordHz.GetValueOnAnyThread();
	float MaxRecordHz = CVarDemoRecordHz.GetValueOnAnyThread();

	if (MaxRecordHz < MinRecordHz)
	{
		Swap(MinRecordHz, MaxRecordHz);
	}

	FRepActorsParams Params
	(
		ClientConnection,
		CVarUseAdaptiveReplayUpdateFrequency.GetValueOnAnyThread() > 0,
		!bDoFindActorChannelEarly,
		!bDoCheckDormancyEarly,
		MinRecordHz,
		MaxRecordHz,
		ServerTickTime,
		RecordFrameStartTime,
		RecordTimeLimit,
		RecordTimeLimit * RecordDestructionInfoReplicationTimeSlice
	);

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Replay actor replication time"), STAT_ReplayReplicateActors, STATGROUP_Net);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ReplayReplicateActors);

		if (HasLevelStreamingFixes())
		{
			const FDemoActorPriority* Priorities = PrioritizedActors.GetData();

			// Split per level		
			for (const FLevelnterval& Interval : LevelIntervals)
			{
				if (!ReplicatePrioritizedActors(&Priorities[Interval.StartIndex], Interval.Count, Params))
				{
					break;
				}
			}
		}
		else
		{
			ReplicatePrioritizedActors(PrioritizedActors.GetData(), PrioritizedActors.Num(), Params);
		}
	}

	CSV_CUSTOM_STAT(Demo, DemoNumReplicatedActors, Params.NumActorsReplicated, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Demo, DemoNumReplicatedDestructionInfos, Params.NumDestructionInfosReplicated, ECsvCustomStatOp::Set);

	FReplayHelper::FlushNetChecked(*ClientConnection);

	WriteDemoFrameFromQueuedDemoPackets(*FileAr, ReplayHelper.QueuedDemoPackets, GetDemoCurrentTime(), EWriteDemoFrameFlags::None);

	const float ReplicatedPercent = NumPrioritizedActors != 0 ? (float)(Params.NumActorsReplicated + Params.NumDestructionInfosReplicated) / (float)NumPrioritizedActors : 1.0f;
	AdjustConsiderTime(ReplicatedPercent);
	LastReplayFrameFidelity = ReplicatedPercent;
}

bool UDemoNetDriver::ReplicatePrioritizedActor(const FActorPriority& ActorPriority, FRepActorsParams& Params)
{
	FNetworkObjectInfo* ActorInfo = ActorPriority.ActorInfo;
	FActorDestructionInfo* DestructionInfo = ActorPriority.DestructionInfo;

	const double RecordStartTimeSeconds = FPlatformTime::Seconds();

	const bool bDoFindActorChannel = Params.bDoFindActorChannel;
	const bool bDoCheckDormancy = Params.bDoCheckDormancy;
	const bool bDestructionInfo = DestructionInfo != nullptr && ActorInfo == nullptr;
	const bool bActorInfo = ActorInfo != nullptr && DestructionInfo == nullptr;

	// Deletion entry
	if (bDestructionInfo)
	{
		// only process destruction infos if we're below the time limit
		const bool bHasDestructionInfoTimeBeenExhausted = (Params.DestructionInfoTimeLimitSeconds > 0.0) && (Params.TotalDestructionInfoRecordTime < Params.DestructionInfoTimeLimitSeconds);
		if (!bHasDestructionInfoTimeBeenExhausted)
		{
			++Params.NumDestructionInfosReplicated;

			UActorChannel* Channel = (UActorChannel*)Params.Connection->CreateChannelByName(NAME_Actor, EChannelCreateFlags::OpenedLocally);
			if (Channel)
			{
				UE_LOGF(LogDemo, Verbose, "TickDemoRecord creating destroy channel for NetGUID <%ls,%ls> Priority: %d", *DestructionInfo->NetGUID.ToString(), *DestructionInfo->PathName, ActorPriority.Priority);

				SendDestructionInfo(Params.Connection, DestructionInfo);

				// Remove from connection's to-be-destroyed list (close bunch is reliable, so it will make it there)
				Params.Connection->RemoveDestructionInfo(DestructionInfo);

				// calling conditional cleanup now allows the channel to be returned to any pools and reused immediately
				Channel->ConditionalCleanUp(false, DestructionInfo->Reason);
			}
		}
	}
	else if (bActorInfo)
	{
		constexpr bool bGetObjectEvenIfPendingKill = true;
		if (ensureMsgf(ActorInfo->WeakActor.IsValid(), TEXT("Trying to record an invalid actor `%s`. Skipping."), ToCStr(GetNameSafe(ActorInfo->WeakActor.Get(bGetObjectEvenIfPendingKill)))))
		{
			++Params.NumActorsReplicated;

			AActor* Actor = ActorInfo->Actor;

			if (bDoCheckDormancy)
			{
				UActorChannel* Channel = (bDoFindActorChannel ? Params.Connection->FindActorChannelRef(ActorInfo->WeakActor) : ActorPriority.Channel);
				if (Channel && ShouldActorGoDormantForDemo(Actor, Channel))
				{
					// Either shouldn't go dormant, or is already dormant
					Channel->StartBecomingDormant();
				}
			}

			// Use NetUpdateFrequency for this actor, but clamp it to RECORD_HZ.
			const float ClampedNetUpdateFrequency = FMath::Clamp(Actor->GetNetUpdateFrequency(), Params.MinRecordHz, Params.MaxRecordHz);
			const double NetUpdateDelay = 1.0 / ClampedNetUpdateFrequency;

			// Set defaults if this actor is replicating for first time
			if (ActorInfo->LastNetReplicateTime == 0)
			{
				ActorInfo->LastNetReplicateTime = GetDemoCurrentTime();
				ActorInfo->OptimalNetUpdateDelta = NetUpdateDelay;
			}

			const float LastReplicateDelta = static_cast<float>(GetDemoCurrentTime() - ActorInfo->LastNetReplicateTime);

			// Calculate min delta (max rate actor will update), and max delta (slowest rate actor will update)
			const float MinOptimalDelta = NetUpdateDelay;										// Don't go faster than NetUpdateFrequency
			const float MinNetUpdateFrequency = (Actor->GetMinNetUpdateFrequency() == 0.0f) ? 2.0f : Actor->GetMinNetUpdateFrequency();
			const float MaxOptimalDelta = FMath::Max(1.0f / MinNetUpdateFrequency, MinOptimalDelta);	// Don't go slower than MinNetUpdateFrequency (or NetUpdateFrequency if it's slower)

			const float ScaleDownStartTime = 2.0f;
			const float ScaleDownTimeRange = 5.0f;

			if (LastReplicateDelta > ScaleDownStartTime)
			{
				// Interpolate between MinOptimalDelta/MaxOptimalDelta based on how long it's been since this actor actually sent anything
				const float Alpha = FMath::Clamp((LastReplicateDelta - ScaleDownStartTime) / ScaleDownTimeRange, 0.0f, 1.0f);
				ActorInfo->OptimalNetUpdateDelta = FMath::Lerp(MinOptimalDelta, MaxOptimalDelta, Alpha);
			}

			const double NextUpdateDelta = Params.bUseAdapativeNetFrequency ? ActorInfo->OptimalNetUpdateDelta : NetUpdateDelay;

			// Account for being fractionally into the next frame
			// But don't be more than a fraction of a frame behind either (we don't want to do catch-up frames when there is a long delay)
			const double ExtraTime = GetDemoCurrentTime() - ActorInfo->NextUpdateTime;
			const double ClampedExtraTime = FMath::Clamp(ExtraTime, 0.0, NetUpdateDelay);

			// Try to spread the updates across multiple frames to smooth out spikes.
			ActorInfo->NextUpdateTime = (GetDemoCurrentTime() + NextUpdateDelta - ClampedExtraTime + ((UpdateDelayRandomStream.FRand() - 0.5) * Params.ServerTickTime));

			const bool bDidReplicateActor = DemoReplicateActor(Actor, Params.Connection, false);

			if (bDidReplicateActor)
			{
				// Choose an optimal time, we choose 70% of the actual rate to allow frequency to go up if needed
				ActorInfo->OptimalNetUpdateDelta = FMath::Clamp(LastReplicateDelta * 0.7f, MinOptimalDelta, MaxOptimalDelta);
				ActorInfo->LastNetReplicateTime = GetDemoCurrentTime();
			}
		}
	}
	else
	{
		UE_LOGF(LogDemo, Warning, "TickDemoRecord: prioritized actor entry should have either an actor or a destruction info");
	}

	// Make sure we're under the desired recording time quota, if any.
	if (Params.TimeLimitSeconds > 0.f)
	{
		const double RecordEndTimeSeconds = FPlatformTime::Seconds();
		const double RecordTimeSeconds = RecordEndTimeSeconds - RecordStartTimeSeconds;

		if (bDestructionInfo)
		{
			Params.TotalDestructionInfoRecordTime += RecordTimeSeconds;
		}

		if ((ActorInfo && ActorInfo->Actor) && (RecordTimeSeconds > (Params.TimeLimitSeconds * 0.95f)))
		{
			UE_LOGF(LogDemo, Verbose, "Actor %ls took more than 95%% of maximum desired recording time. Actor: %.3fms. Max: %.3fms.",
				*ActorInfo->Actor->GetName(), RecordTimeSeconds * 1000.f, MaxDesiredRecordTimeMS);
		}

		const double TotalRecordTimeSeconds = (RecordEndTimeSeconds - Params.ReplicationStartTimeSeconds);

		if (TotalRecordTimeSeconds > Params.TimeLimitSeconds)
		{
			BudgetLogHelper->MarkFrameOverBudget(
				FDemoBudgetLogHelper::Replication,
				TEXT("Exceeded maximum desired recording time (during Actor Replication).  Max: %.3fms."),
				MaxDesiredRecordTimeMS);

			return false;
		}
	}

	return true;
}

bool UDemoNetDriver::ReplicatePrioritizedActors(const FDemoActorPriority* ActorsToReplicate, uint32 Count, FRepActorsParams& Params)
{
	bool bTimeRemaining = true;
	uint32 NumProcessed = 0;
	for (; NumProcessed < Count; ++NumProcessed)
	{
		const FActorPriority& ActorPriority = ActorsToReplicate[NumProcessed].ActorPriority;
		bTimeRemaining = ReplicatePrioritizedActor(ActorPriority, Params);
		if (!bTimeRemaining)
		{
			++NumProcessed;
			break;
		}
	}

	return bTimeRemaining;
}

void UDemoNetDriver::PauseChannels(const bool bPause)
{
	if (bPause == bChannelsArePaused)
	{
		return;
	}

	if (CVarDemoInternalPauseChannels.GetValueOnAnyThread() > 0)
	{
		// Pause all non player controller actors
		for (int32 i = ServerConnection->OpenChannels.Num() - 1; i >= 0; i--)
		{
			UChannel* OpenChannel = ServerConnection->OpenChannels[i];

			UActorChannel* ActorChannel = Cast<UActorChannel>(OpenChannel);

			if (ActorChannel == nullptr)
			{
				continue;
			}

			ActorChannel->CustomTimeDilation = bPause ? 0.0f : 1.0f;

			if (ActorChannel->GetActor() == nullptr || SpectatorControllers.Contains(ActorChannel->GetActor()))
			{
				continue;
			}

			// Better way to pause each actor?
			ActorChannel->GetActor()->CustomTimeDilation = ActorChannel->CustomTimeDilation;
		}
	}

	bChannelsArePaused = bPause;

	UE_LOGF(LogDemo, Verbose, "PauseChannels: %d", bChannelsArePaused);
	FNetworkReplayDelegates::OnPauseChannelsChanged.Broadcast(World, bChannelsArePaused);
}

bool UDemoNetDriver::ReadDemoFrameIntoPlaybackPackets(FArchive& Ar, TArray<FPlaybackPacket>& InPlaybackPackets, const bool bForLevelFastForward, float* OutTime)
{
	return ReplayHelper.ReadDemoFrame(ServerConnection, Ar, InPlaybackPackets, bForLevelFastForward, MaxArchiveReadPos, OutTime);
}

void UDemoNetDriver::ProcessSeamlessTravel(int32 LevelIndex)
{
	// Destroy all player controllers since FSeamlessTravelHandler will not destroy them.
	TArray<AController*> Controllers;
	for (FConstControllerIterator Iterator = World->GetControllerIterator(); Iterator; ++Iterator)
	{
		Controllers.Add(Iterator->Get());
	}

	// Clean up any splitscreen spectators if we have them.
	// Let the destroy below handle deletion of the objects.
	if (SpectatorControllers.Num() > 1)
	{
		CleanUpSplitscreenConnections(false);
	}

	for (AController* Controller : Controllers)
	{
		if (Controller)
		{
			// bNetForce is true so that the replicated spectator player controller will
			// be destroyed as well.
			Controller->Destroy(true);

			// If we can, remove the spectator here as well.
			SpectatorControllers.Remove(Cast<APlayerController>(Controller));
		}
	}

	SpectatorControllers.Empty();

	// Set this to nullptr since we just destroyed it.
	SpectatorController = nullptr;

	if (ReplayHelper.PlaybackDemoHeader.LevelNamesAndTimes.IsValidIndex(LevelIndex))
	{
		World->SeamlessTravel(ReplayHelper.PlaybackDemoHeader.LevelNamesAndTimes[LevelIndex].LevelName, true);
	}
	else
	{
		// If we're watching a live replay, it's probable that the header has been updated with the level added,
		// so we need to download it again before proceeding.
		bIsWaitingForHeaderDownload = true;
		ReplayHelper.ReplayStreamer->DownloadHeader(FDownloadHeaderCallback::CreateUObject(this, &UDemoNetDriver::OnRefreshHeaderCompletePrivate, LevelIndex));
	}
}

void UDemoNetDriver::OnRefreshHeaderCompletePrivate(const FDownloadHeaderResult& Result, int32 LevelIndex)
{
	bIsWaitingForHeaderDownload = false;

	if (Result.WasSuccessful())
	{
		FString Error;
		if (ReplayHelper.ReadPlaybackDemoHeader(Error))
		{
			if (ReplayHelper.PlaybackDemoHeader.LevelNamesAndTimes.IsValidIndex(LevelIndex))
			{
				ProcessSeamlessTravel(LevelIndex);
			}
			else
			{
				UE_LOGF(LogDemo, Error, "UDemoNetDriver::OnDownloadHeaderComplete: LevelIndex %d not in range of level names of size: %d", LevelIndex, ReplayHelper.PlaybackDemoHeader.LevelNamesAndTimes.Num());
				ReplayHelper.NotifyReplayError(EReplayResult::Corrupt);
			}
		}
		else
		{
			UE_LOGF(LogDemo, Error, "UDemoNetDriver::OnDownloadHeaderComplete: ReadPlaybackDemoHeader header failed with error %ls.", *Error);
			// ReadPlaybackDemoHeader will have already called NotifyPlaybackError internally on failure
		}
	}
	else
	{
		UE_LOGF(LogDemo, Error, "UDemoNetDriver::OnDownloadHeaderComplete: Downloading header failed.");
		ReplayHelper.NotifyReplayError(EReplayResult::Corrupt);
	}
}

bool UDemoNetDriver::ConditionallyReadDemoFrameIntoPlaybackPackets(FArchive& Ar)
{
	if (PlaybackPackets.Num() > 0)
	{
		const float MAX_PLAYBACK_BUFFER_SECONDS = 5.0f;

		const FPlaybackPacket& LastPacket = PlaybackPackets.Last();
		const float CurrentTime = GetDemoCurrentTime();

		if ((LastPacket.TimeSeconds > CurrentTime) && ((LastPacket.TimeSeconds - CurrentTime) > MAX_PLAYBACK_BUFFER_SECONDS))
		{
			return false;	// Don't buffer more than MAX_PLAYBACK_BUFFER_SECONDS worth of frames
		}
	}

	if (!ReadDemoFrameIntoPlaybackPackets(Ar))
	{
		return false;
	}

	return true;
}

bool UDemoNetDriver::ShouldSkipPlaybackPacket(const FPlaybackPacket& Packet)
{
	if (HasLevelStreamingFixes() && Packet.SeenLevelIndex != 0)
	{
		if (ReplayHelper.SeenLevelStatuses.IsValidIndex(Packet.SeenLevelIndex - 1))
		{
			// Flag the status as being seen, since we're potentially going to process it.
			// We need to skip processing if it's not ready (in that case, we'll do a fast-forward).
			FReplayHelper::FLevelStatus& LevelStatus = ReplayHelper.GetLevelStatus(Packet.SeenLevelIndex);
			LevelStatus.bHasBeenSeen = true;
			return !LevelStatus.bIsReady;
		}
		else
		{
			UE_LOGF(LogDemo, Warning, "ShouldSkipPlaybackPacket encountered a packet with an invalid seen level index.");
		}
	}

	return false;
}

bool UDemoNetDriver::ConditionallyProcessPlaybackPackets()
{
	if (!PlaybackPackets.IsValidIndex(PlaybackPacketIndex))
	{
		PauseChannels(true);
		return false;
	}

	const FPlaybackPacket& CurPacket = PlaybackPackets[PlaybackPacketIndex];
	if (GetDemoCurrentTime() < CurPacket.TimeSeconds)
	{
		// Not enough time has passed to read another frame
		return false;
	}

	if (CurPacket.LevelIndex != GetCurrentLevelIndex())
	{
		World->GetGameInstance()->OnSeamlessTravelDuringReplay();
		SetCurrentLevelIndex(CurPacket.LevelIndex);
		ProcessSeamlessTravel(GetCurrentLevelIndex());
		return false;
	}

	++PlaybackPacketIndex;
	return ProcessPacket(CurPacket);
}

void UDemoNetDriver::ProcessAllPlaybackPackets()
{
	ProcessPlaybackPackets(PlaybackPackets);
	PlaybackPackets.Empty();
	// this call is used for checkpoint loading, so not dealing with per frame data
	ReplayHelper.PlaybackFrames.Empty();
}

void UDemoNetDriver::ProcessPlaybackPackets(TArrayView<FPlaybackPacket> Packets)
{
	if (Packets.Num() > 0)
	{
		for (const FPlaybackPacket& PlaybackPacket : Packets)
		{
			ProcessPacket(PlaybackPacket);
		}

		LastProcessedPacketTime = Packets.Last().TimeSeconds;
	}
}

bool UDemoNetDriver::ProcessPacket(const uint8* Data, int32 Count)
{
	PauseChannels(false);

	if (ServerConnection != nullptr )
	{
		// Process incoming packet.
		// ReceivedRawPacket shouldn't change any data, so const_cast should be safe.
		ServerConnection->ReceivedRawPacket(const_cast<uint8*>(Data), Count);
	}

	if (ServerConnection == nullptr || ServerConnection->GetConnectionState() == USOCK_Closed)
	{
		// Something we received resulted in the demo being stopped
		UE_LOGF(LogDemo, Error, "UDemoNetDriver::ProcessPacket: ReceivedRawPacket closed connection");
		ReplayHelper.NotifyReplayError(EReplayResult::ConnectionClosed);
		return false;
	}

	return true;
}

void UDemoNetDriver::WriteDemoFrameFromQueuedDemoPackets(FArchive& Ar, TArray<FQueuedDemoPacket>& QueuedPackets, float FrameTime, EWriteDemoFrameFlags Flags)
{
	ReplayHelper.WriteDemoFrame(ClientConnections[0], Ar, QueuedPackets, FrameTime, Flags);
}

void UDemoNetDriver::WritePacket(FArchive& Ar, uint8* Data, int32 Count)
{
	ReplayHelper.WritePacket(Ar, Data, Count);
}

void UDemoNetDriver::SkipTime(const float InTimeToSkip)
{
	if (IsNamedTaskInQueue(ReplayTaskNames::SkipTimeInSecondsTask))
	{
		return;		// Don't allow time skipping if we already are
	}

	AddReplayTask(new FSkipTimeInSecondsTask(this, InTimeToSkip));
}

void UDemoNetDriver::SkipTimeInternal(const float SecondsToSkip, const bool InFastForward, const bool InIsForCheckpoint)
{
	check(!bIsFastForwarding);				// Can only do one of these at a time (use tasks to gate this)
	check(!bIsFastForwardingForCheckpoint);	// Can only do one of these at a time (use tasks to gate this)

	SavedSecondsToSkip = SecondsToSkip;

	SetDemoCurrentTime(FMath::Clamp(GetDemoCurrentTime() + SecondsToSkip, 0.0f, GetDemoTotalTime() - 0.01f));

	bIsFastForwarding				= InFastForward;
	bIsFastForwardingForCheckpoint	= InIsForCheckpoint;
}

void UDemoNetDriver::GotoTimeInSeconds(const float TimeInSeconds, const FOnGotoTimeDelegate& InOnGotoTimeDelegate)
{
	OnGotoTimeDelegate_Transient = InOnGotoTimeDelegate;

	if (IsNamedTaskInQueue(ReplayTaskNames::GotoTimeInSecondsTask) || bIsFastForwarding)
	{
		NotifyGotoTimeFinished(false);
		return;		// Don't allow scrubbing if we already are
	}

	UE_LOGF(LogDemo, Log, "GotoTimeInSeconds: %2.2f", TimeInSeconds);

	AddReplayTask(new FGotoTimeInSecondsTask(this, TimeInSeconds));
}

void UDemoNetDriver::JumpToEndOfLiveReplay()
{
	UE_LOGF(LogDemo, Log, "UDemoNetDriver::JumpToEndOfLiveReplay.");

	const uint32 TotalDemoTimeInMS = GetReplayStreamer()->GetTotalDemoTime();

	SetDemoTotalTime((float)TotalDemoTimeInMS / 1000.0f);

	const uint32 BufferInMS = 5 * 1000;

	const uint32 JoinTimeInMS = FMath::Max((uint32)0, GetReplayStreamer()->GetTotalDemoTime() - BufferInMS);

	if (JoinTimeInMS > 0)
	{
		GotoTimeInSeconds((float)JoinTimeInMS / 1000.0f);
	}
}

void UDemoNetDriver::AddUserToReplay(const FString& UserString)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->AddUserToReplay(UserString);
	}
}

#if DEMO_CSV_PROFILING_HELPERS_ENABLED
struct FCsvDemoSettings
{
	FCsvDemoSettings() 
		: bCaptureCsv(false)
		, StartTime(-1)
		, EndTime(-1)
		, FrameCount(0)
		, bStopAfterProfile(false)
		, bStopCsvAtReplayEnd(false)
	{}
	bool bCaptureCsv;
	int32 StartTime;
	int32 EndTime;
	int32 FrameCount;
	bool bStopAfterProfile;
	bool bStopCsvAtReplayEnd;
};

static FCsvDemoSettings GetCsvDemoSettings()
{
	FCsvDemoSettings Settings = {};
	Settings.bCaptureCsv = FParse::Value(FCommandLine::Get(), TEXT("-csvdemostarttime="), Settings.StartTime);
	if (Settings.bCaptureCsv)
	{
		if (!FParse::Value(FCommandLine::Get(), TEXT("-csvdemoendtime="), Settings.EndTime))
		{
			Settings.EndTime = -1.0;
		}
		if (!FParse::Value(FCommandLine::Get(), TEXT("-csvdemoframecount="), Settings.FrameCount))
		{
			Settings.FrameCount = -1;
		}
	}
	Settings.bStopAfterProfile = FParse::Param(FCommandLine::Get(), TEXT("csvDemoStopAfterProfile"));
	Settings.bStopCsvAtReplayEnd = FParse::Param(FCommandLine::Get(), TEXT("csvDemoStopCsvAtReplayEnd"));
	return Settings;
}
#endif // DEMO_CSV_PROFILING_HELPERS_ENABLED

class FDemoNetDriverReplayPlaylistHelper
{
private:

	friend class UDemoNetDriver;

	static void RestartPlaylist(FReplayPlaylistTracker& ToRestart)
	{
		ToRestart.Restart();
	}
};

void UDemoNetDriver::TickDemoPlayback(float DeltaSeconds)
{
	LLM_SCOPE(ELLMTag::Replays);
	SCOPED_NAMED_EVENT(UDemoNetDriver_TickDemoPlayback, FColor::Purple);
	if (World && World->IsInSeamlessTravel())
	{
		return;
	}

#if DEMO_CSV_PROFILING_HELPERS_ENABLED
	static FCsvDemoSettings CsvDemoSettings = GetCsvDemoSettings();
	{
		if (CsvDemoSettings.bCaptureCsv)
		{
			bool bDoCapture = IsPlaying()
				&& GetDemoCurrentTime() >= CsvDemoSettings.StartTime
				&& ((GetDemoCurrentTime() <= CsvDemoSettings.EndTime) || (CsvDemoSettings.EndTime < 0));

			static bool bStartedCsvRecording = false;
			if (!bStartedCsvRecording && bDoCapture)
			{
				FCsvProfiler::Get()->BeginCapture(CsvDemoSettings.FrameCount);
				bStartedCsvRecording = true;
			}
			else if (bStartedCsvRecording && !bDoCapture)
			{
				FCsvProfiler::Get()->EndCapture();
				bStartedCsvRecording = false;
			}
		}
	}
#endif // DEMO_CSV_PROFILING_HELPERS_ENABLED

	if (!IsPlaying())
	{
		return;
	}
	CSV_CUSTOM_STAT(Demo, Time, GetDemoCurrentTime()+DeltaSeconds, ECsvCustomStatOp::Set);

	// This will be true when watching a live replay and we're grabbing an up to date header.
	// In that case, we want to pause playback until we can actually travel.
	if (bIsWaitingForHeaderDownload)
	{
		return;
	}

	if (CVarForceDisableAsyncPackageMapLoading.GetValueOnGameThread() > 0)
	{
		GetNetGuidCache()->SetAsyncLoadMode(FNetGUIDCache::EAsyncLoadMode::ForceDisable);
	}
	else
	{
		GetNetGuidCache()->SetAsyncLoadMode(FNetGUIDCache::EAsyncLoadMode::UseCVar);
	}

	if (CVarGotoTimeInSeconds.GetValueOnGameThread() >= 0.0f)
	{
		GotoTimeInSeconds(CVarGotoTimeInSeconds.GetValueOnGameThread());
		CVarGotoTimeInSeconds.AsVariable()->Set(TEXT( "-1" ), ECVF_SetByConsole);
	}

	if (FMath::Abs(CVarDemoSkipTime.GetValueOnGameThread()) > 0.0f)
	{
		// Just overwrite existing value, cvar wins in this case
		GotoTimeInSeconds(GetDemoCurrentTime() + CVarDemoSkipTime.GetValueOnGameThread());
		CVarDemoSkipTime.AsVariable()->Set(TEXT("0"), ECVF_SetByConsole);
	}

	// Before we update tasks or move the demo time forward, see if there are any new sublevels that
	// need to be fast forwarded.
	PrepFastForwardLevels();

	// Update total demo time
	if (ReplayHelper.ReplayStreamer->GetTotalDemoTime() > 0)
	{
		SetDemoTotalTime((float)ReplayHelper.ReplayStreamer->GetTotalDemoTime() / 1000.0f);
	}

	if (!ProcessReplayTasks())
	{
		// We're busy processing tasks, return
		return;
	}

	// If we don't have data on frame 0 wait until we have it
	if (!GetReplayStreamer()->IsDataAvailable() && ReplayHelper.DemoFrameNum == 0)
	{
		return;
	}

	// If the ExitAfterReplay option is set, automatically shut down at the end of the replay.
	// Use AtEnd() of the archive instead of checking DemoCurrentTime/DemoTotalTime, because the DemoCurrentTime may never catch up to DemoTotalTime.
	if (FArchive* const StreamingArchive = ReplayHelper.ReplayStreamer->GetStreamingArchive())
	{
		bool bIsAtEnd = StreamingArchive->AtEnd() && (PlaybackPackets.Num() == 0 || (GetDemoCurrentTime() + DeltaSeconds >= GetDemoTotalTime()));
#if DEMO_CSV_PROFILING_HELPERS_ENABLED
		bool bCsvIsCapturing = FCsvProfiler::Get()->IsCapturing();
		static bool bCsvProfilingEnabledPreviousTick = bCsvIsCapturing;
	    if (CsvDemoSettings.bStopAfterProfile && !bCsvIsCapturing && bCsvProfilingEnabledPreviousTick)
	    {
			bIsAtEnd = true;
	    }
		if (bIsAtEnd && bCsvIsCapturing && CsvDemoSettings.bStopCsvAtReplayEnd)
		{
			FCsvProfiler::Get()->EndCapture();
		}
		bCsvProfilingEnabledPreviousTick = bCsvIsCapturing;
#endif
		if (!ReplayHelper.ReplayStreamer->IsLive() && bIsAtEnd)
		{
			FNetworkReplayDelegates::OnReplayPlaybackComplete.Broadcast(World);

			FReplayPlaylistTracker* LocalPlaylistTracker = PlaylistTracker.Get();

			CSV_METADATA(TEXT("ReplayID"), nullptr);

			// checking against 1 so the count will mean total number of playthroughs, not additional loops
			if (GDemoLoopCount > 1)
			{
				if (LocalPlaylistTracker)
				{
					if (LocalPlaylistTracker->IsOnLastReplay())
					{
						--GDemoLoopCount;
						FDemoNetDriverReplayPlaylistHelper::RestartPlaylist(*LocalPlaylistTracker);
					}
				}
				else
				{
					--GDemoLoopCount;
					GotoTimeInSeconds(0.0f);
				}	
			}
			else
			{
				if (FParse::Param(FCommandLine::Get(), TEXT("ExitAfterReplay")) && (!LocalPlaylistTracker || LocalPlaylistTracker->IsOnLastReplay()))
				{
					FPlatformMisc::RequestExit(false, TEXT("DemoNetDriver"));
				}
				else
				{
					if (CVarLoopDemo.GetValueOnGameThread() > 0)
					{
						if (!LocalPlaylistTracker)
						{
							GotoTimeInSeconds(0.0f);
						}
						else if (LocalPlaylistTracker->IsOnLastReplay())
						{
							FDemoNetDriverReplayPlaylistHelper::RestartPlaylist(*LocalPlaylistTracker);
						}
					}
				}
			}
		}
	}

	// Advance demo time by seconds passed if we're not paused
	if (World && World->GetWorldSettings() && World->GetWorldSettings()->GetPauserPlayerState() == nullptr)
	{
		SetDemoCurrentTime(GetDemoCurrentTime() + DeltaSeconds);
	}

	// Clamp time
	SetDemoCurrentTime(FMath::Clamp(GetDemoCurrentTime(), 0.0f, GetDemoTotalTime() + 0.01f));

	ReplayHelper.ReplayStreamer->UpdatePlaybackTime(GetDemoCurrentTimeInMS());

	bool bProcessAvailableData = (PlaybackPackets.Num() > 0) || GetReplayStreamer()->IsDataAvailable();
	
	if (CVarFastForwardLevelsPausePlayback.GetValueOnAnyThread() == 0)
	{
		const uint32 DemoCurrentTimeInMS = GetDemoCurrentTimeInMS();
		bProcessAvailableData = bProcessAvailableData || GetReplayStreamer()->IsDataAvailableForTimeRange(DemoCurrentTimeInMS, DemoCurrentTimeInMS);
	}

	// Make sure there is data available to read
	// If we're at the end of the demo, just pause channels and return
	if (bProcessAvailableData)
	{
		// we either have packets to process or data available to read
		PauseChannels(false);
	}
	else
	{
		PauseChannels(true);
		return;
	}

	// Speculatively grab seconds now in case we need it to get the time it took to fast forward
	const double FastForwardStartSeconds = FPlatformTime::Seconds();

	if (FArchive* const StreamingArchive = GetReplayStreamer()->GetStreamingArchive())
	{
		ReplayHelper.SetPlaybackNetworkVersions(*StreamingArchive);
	}

	// Buffer up demo frames until we have enough time built-up
	while (ConditionallyReadDemoFrameIntoPlaybackPackets(*GetReplayStreamer()->GetStreamingArchive()))
	{
	}

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("TickDemoPlayback_ProcessPackets"), TickDemoPlayback_ProcessPackets, STATGROUP_Net);

		// Process packets until we are caught up (this implicitly handles fast forward if DemoCurrentTime past many frames)
		while (ConditionallyProcessPlaybackPackets())
		{
			ReplayHelper.DemoFrameNum++;
		}

		if (PlaybackPacketIndex > 0)
		{
			// Remove all packets that were processed
			// At this point, PlaybackPacketIndex will actually be the number of packets we've processed,
			// as it points to the "next" index we would otherwise have processed.
			LastProcessedPacketTime = PlaybackPackets[PlaybackPacketIndex - 1].TimeSeconds;

			PlaybackPackets.RemoveAt(0, PlaybackPacketIndex);
			PlaybackPacketIndex = 0;
		}

		// Process playback frames
		for (auto FrameIt = ReplayHelper.PlaybackFrames.CreateIterator(); FrameIt; ++FrameIt)
		{
			if (FrameIt.Key() <= GetDemoCurrentTime())
			{
				if (!bIsFastForwarding)
				{
					FNetworkReplayDelegates::OnProcessGameSpecificFrameData.Broadcast(World, FrameIt.Key(), FrameIt.Value());
				}
	
				FrameIt.RemoveCurrent();
			}
		}
	}

	// Finalize any fast forward stuff that needs to happen
	if (bIsFastForwarding)
	{
		FinalizeFastForward(FastForwardStartSeconds);
	}
}

void UDemoNetDriver::FinalizeFastForward(const double StartTime)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Demo_FinalizeFastForward"), Demo_FinalizeFastForward, STATGROUP_Net);

	TGuardValue<bool> FinalizingFastForward(bIsFinalizingFastForward, true);

	// This must be set before we CallRepNotifies or they might be skipped again
	bIsFastForwarding = false;

	AGameStateBase* const GameState = World != nullptr ? World->GetGameState() : nullptr;

	// Make sure that we delete any Rewind actors that aren't valid anymore.
	if (bIsFastForwardingForCheckpoint)
	{
		CleanupOutstandingRewindActors();
	}	

	// Correct server world time for fast-forwarding after a checkpoint
	if (GameState != nullptr)
	{
		if (bIsFastForwardingForCheckpoint)
		{
			const double PostCheckpointServerTime = SavedReplicatedWorldTimeSeconds + SavedSecondsToSkip;
			GameState->ReplicatedWorldTimeSecondsDouble = PostCheckpointServerTime;
		}

		// Correct the ServerWorldTimeSecondsDelta
		GameState->OnRep_ReplicatedWorldTimeSecondsDouble();
	}

	if (ServerConnection != nullptr && bIsFastForwardingForCheckpoint)
	{
		// Make a pass at OnReps for startup actors, since they were skipped during checkpoint loading.
		// At this point the shadow state of these actors should be the actual state from before the checkpoint,
		// and the current state is the CDO state evolved by any changes that occurred during checkpoint loading and fast-forwarding.
		for (UChannel* Channel : ServerConnection->OpenChannels)
		{
			UActorChannel* const ActorChannel = Cast<UActorChannel>(Channel);
			if (ActorChannel == nullptr)
			{
				continue;
			}

			const AActor* const Actor = ActorChannel->GetActor();
			if (Actor == nullptr)
			{
				continue;
			}

			if (Actor->IsNetStartupActor())
			{
				DiffActorProperties(ActorChannel);
			}
		}
	}

	// Flush all pending RepNotifies that were built up during the fast-forward.
	if (ServerConnection != nullptr)
	{
		// OnReps can cause the channel map to be modified. Make a copy and lookup the object in the active channel map.
		const FActorChannelMap& ChannelMap = ServerConnection->ActorChannelMap();
		const FActorChannelMap ChannelMapCopy = ChannelMap;
		for (const TPair<TWeakObjectPtr<AActor>, UActorChannel*>& ChannelPair : ChannelMapCopy)
		{
			const TWeakObjectPtr<AActor>& Actor = ChannelPair.Key;
			if (!Actor.IsValid())
			{
				continue;
			}

			// Look up the actor in the original channel map
			if (const UActorChannel*const* Channel = ChannelMap.Find(Actor))
			{
				for (const TPair<UObject*, TSharedRef<FObjectReplicator>>& ReplicatorPair : (*Channel)->ReplicationMap)
				{
					ReplicatorPair.Value->CallRepNotifies(true);
				}
			}
		}

		auto CallRepNotifies = [](FObjectKey OwnerActorKey, FObjectKey ObjectKey, const TSharedRef<FObjectReplicator>& ReplicatorRef)
		{
			ReplicatorRef->CallRepNotifies(true);
		};

		ServerConnection->ExecuteOnAllDormantReplicators(CallRepNotifies);
	}

	// We may have been fast-forwarding immediately after loading a checkpoint
	// for fine-grained scrubbing. If so, at this point we are no longer loading a checkpoint.
	bIsFastForwardingForCheckpoint = false;

	// Reset the never-queue GUID list, we'll rebuild it
	NonQueuedGUIDsForScrubbing.Reset();

	const double FastForwardTotalSeconds = FPlatformTime::Seconds() - StartTime;

	NotifyGotoTimeFinished(true);

	UE_LOGF(LogDemo, Log, "Fast forward took %.2f seconds.", FastForwardTotalSeconds);
}

void UDemoNetDriver::SpawnDemoRecSpectator(UNetConnection* Connection, const FURL& ListenURL)
{
	SpectatorController = ReplayHelper.CreateSpectatorController(Connection);
	if (SpectatorController)
	{
		SpectatorControllers.Add(SpectatorController);
	}
}

bool UDemoNetDriver::SpawnSplitscreenViewer(ULocalPlayer* NewPlayer, UWorld* InWorld)
{
	if (NewPlayer == nullptr || InWorld == nullptr)
	{
		UE_LOGF(LogDemo, Warning, "UDemoNetDriver::SpawnSplitscreenViewer: Local Player or World is invalid!");
		return false;
	}

	if (ClientConnections.Num() == 0 && ServerConnection == nullptr)
	{
		UE_LOGF(LogDemo, Error, "UDemoNetDriver::SpawnSplitscreenViewer: This netdriver has no demo connection data");
		return false;
	}

	UNetConnection* ChildConnection = CreateChild((ClientConnections.Num() > 0) ? ClientConnections[0] : ServerConnection);

	APlayerController* NewSplitscreenController = ReplayHelper.CreateSpectatorController(ChildConnection);
	if (NewSplitscreenController == nullptr)
	{
		UE_LOGF(LogDemo, Warning, "UDemoNetDriver::SpawnSplitscreenViewer: Unable to create new splitscreen controller");
		return false;
	}

	// Link this spectator to the given local player, as this will facilitate spectator pawn creation 
	// (spectator pawns only create if the controller is linked to a local player)
	NewSplitscreenController->Player = NewPlayer;
	NewSplitscreenController->NetPlayerIndex = GEngine->GetGamePlayers(InWorld).Find(NewPlayer);

	// Create the Pawn
	NewSplitscreenController->ChangeState(NAME_Spectating);
	NewPlayer->CurrentNetSpeed = 0;

	// Link the local player to the player controller as the local player has been marked as active
	// but without a PlayerController, the player will never be considered "ready" by other systems.
	NewPlayer->PlayerController = NewSplitscreenController;

	// This would typically be set via SetPlayer, but we need to call SetPlayer with the LocalPlayer
	// and not with the child connection, otherwise we never create the input controls we need.
	ChildConnection->PlayerController = NewSplitscreenController;
	ChildConnection->OwningActor = NewSplitscreenController;
	
	// Create input control
	NewSplitscreenController->SetPlayer(NewPlayer);

	// Add to the list
	SpectatorControllers.Add(NewSplitscreenController);

	return true;
}

bool UDemoNetDriver::RemoveSplitscreenViewer(APlayerController* RemovePlayer, bool bMarkOwnerForDeletion)
{
	UE_LOGF(LogDemo, Log, "Attempting to remove splitscreen viewer!");

	if (RemovePlayer && SpectatorControllers.Contains(RemovePlayer) && RemovePlayer != SpectatorController)
	{
		SpectatorControllers.Remove(RemovePlayer);
		UNetConnection* RemovedNetConnection = RemovePlayer->NetConnection;
		if (!bMarkOwnerForDeletion)
		{
			RemovedNetConnection->OwningActor = nullptr;
		}
		RemovedNetConnection->Close();
		RemovedNetConnection->CleanUp();
		RemovePlayer->NetConnection = nullptr;
		return true;
	}

	return false;
}

int32 UDemoNetDriver::CleanUpSplitscreenConnections(bool bDeleteOwner)
{
	int32 NumSplitscreenConnectionsCleaned = 0;

	for (APlayerController* CurController : SpectatorControllers)
	{
		UNetConnection* ControllerNetConnection = (CurController != nullptr) ? ToRawPtr(CurController->NetConnection) : nullptr;
		if (ControllerNetConnection != nullptr && ControllerNetConnection->IsA(UChildConnection::StaticClass()))
		{
			++NumSplitscreenConnectionsCleaned;
			// With this toggled, this prevents actor deletion (which we don't want to do when scrubbing)
			if (!bDeleteOwner)
			{
				ControllerNetConnection->OwningActor = nullptr;
			}
			ControllerNetConnection->Close();
			ControllerNetConnection->CleanUp();
			CurController->NetConnection = nullptr;
		}
	}

	UE_LOGF(LogDemo, Log, "Cleaned up %d splitscreen connections, owner deletion: %ls", NumSplitscreenConnectionsCleaned, bDeleteOwner ? TEXT("enabled") : TEXT("disabled"));

	return NumSplitscreenConnectionsCleaned;
}

void UDemoNetDriver::PauseRecording(const bool bInPauseRecording)
{
	ReplayHelper.bPauseRecording = bInPauseRecording;
}

bool UDemoNetDriver::IsRecordingPaused() const
{
	return ReplayHelper.bPauseRecording;
}

void UDemoNetDriver::ReplayStreamingReady(const FStartStreamingResult& Result)
{
	bIsWaitingForStream = false;
	bWasStartStreamingSuccessful = Result.WasSuccessful();

	if (!bWasStartStreamingSuccessful)
	{
		if (Result.bRecording)
		{
			ReplayHelper.NotifyReplayError(EReplayResult::StreamerError);
		}
		else
		{
			ReplayHelper.NotifyReplayError(EReplayResult::ReplayNotFound);
		}
		return;
	}

	if (!Result.bRecording)
	{
		FString Error;
		
		const double StartTime = FPlatformTime::Seconds();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!InitConnectInternal(Error))
		{
			return;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// InitConnectInternal calls ResetDemoState which will reset this, so restore the value
		bWasStartStreamingSuccessful = Result.WasSuccessful();

		const TCHAR* const SkipToLevelIndexOption = ReplayHelper.DemoURL.GetOption(TEXT("SkipToLevelIndex="), nullptr);
		if (SkipToLevelIndexOption)
		{
			int32 Index = FCString::Atoi(SkipToLevelIndexOption);
			if (ReplayHelper.LevelNamesAndTimes.IsValidIndex(Index))
			{
				AddReplayTask(new FGotoTimeInSecondsTask(this, (float)ReplayHelper.LevelNamesAndTimes[Index].LevelChangeTimeInMS / 1000.0f));
			}
			else
			{
				UE_LOGF(LogDemo, Warning, "ReplayStreamingReady: SkipToLevelIndex was invalid: %d", Index);
			}
		}

		if (CVarDemoJumpToEndOfLiveReplay.GetValueOnGameThread() != 0)
		{
			if (ReplayHelper.ReplayStreamer->IsLive() && ReplayHelper.ReplayStreamer->GetTotalDemoTime() > 15 * 1000)
			{
				// If the load time wasn't very long, jump to end now
				// Otherwise, defer it until we have a more recent replay time
				if (FPlatformTime::Seconds() - StartTime < 10)
				{
					JumpToEndOfLiveReplay();
				}
				else
				{
					UE_LOGF(LogDemo, Log, "UDemoNetDriver::ReplayStreamingReady: Deferring checkpoint until next available time.");
					AddReplayTask(new FJumpToLiveReplayTask(this));
				}
			}
		}

		if (UE_LOG_ACTIVE(LogDemo, Log))
		{
			FString HeaderFlags;

			for (uint32 i = 0; i < sizeof(EReplayHeaderFlags) * 8; ++i)
			{
				EReplayHeaderFlags Flag = (EReplayHeaderFlags)(1 << i);

				if (EnumHasAnyFlags(ReplayHelper.PlaybackDemoHeader.HeaderFlags, Flag))
				{
					HeaderFlags += (HeaderFlags.IsEmpty() ? TEXT("") : TEXT("|"));
					HeaderFlags += LexToString(Flag);
				}
			}

			UE_LOGF(LogDemo, Log, "ReplayStreamingReady: playing back replay [%ls] %ls, which was recorded on engine version %ls with flags [%ls]",
				*ReplayHelper.GetPlaybackGuid().ToString(EGuidFormats::Digits), *ReplayHelper.DemoURL.Map, *ReplayHelper.PlaybackDemoHeader.EngineVersion.ToString(), *HeaderFlags);

			if (GetPlaybackReplayVersion() >= FReplayCustomVersion::RecordingMetadata)
			{
				UE_LOGF(LogDemo, Log, "ReplayStreamingReady: replay was recorded with: MinHz: %0.2f MaxHz: %0.2f FrameMS: %0.2f CheckpointMS: %0.2f Platform: [%ls] Config: [%ls] Target: [%ls]",
					ReplayHelper.PlaybackDemoHeader.MinRecordHz, ReplayHelper.PlaybackDemoHeader.MaxRecordHz,
					ReplayHelper.PlaybackDemoHeader.FrameLimitInMS, ReplayHelper.PlaybackDemoHeader.CheckpointLimitInMS,
					*ReplayHelper.PlaybackDemoHeader.Platform, LexToString(ReplayHelper.PlaybackDemoHeader.BuildConfig), LexToString(ReplayHelper.PlaybackDemoHeader.BuildTarget));
			}

			CSV_METADATA(TEXT("ReplayID"), *ReplayHelper.GetPlaybackGuid().ToString(EGuidFormats::Digits));
		}

		// Notify all listeners that a demo is starting
		FNetworkReplayDelegates::OnReplayStarted.Broadcast(World);
	}
}

FReplayExternalDataArray* UDemoNetDriver::GetExternalDataArrayForObject(UObject* Object)
{
	FNetworkGUID NetworkGUID = GetNetGuidCache()->NetGUIDLookup.FindRef(Object);

	if (!NetworkGUID.IsValid())
	{
		return nullptr;
	}

	return ReplayHelper.ExternalDataToObjectMap.Find(NetworkGUID);
}

bool UDemoNetDriver::SetExternalDataForObject(UObject* OwningObject, const uint8* Src, const int32 NumBits)
{
	if (IsRecording())
	{
		// IsRecording verifies that ClientConnections[0] exists
		return ReplayHelper.SetExternalDataForObject(ClientConnections[0], OwningObject, Src, NumBits);
	}

	return false;
}

void UDemoNetDriver::RestoreComponentState(UActorComponent* ActorComp, FRollbackNetStartupActorInfo& RollbackActor)
{
	TSharedPtr<FRepLayout> SubObjLayout = GetObjectClassRepLayout(ActorComp->GetClass());
	if (SubObjLayout.IsValid())
	{
		TSharedPtr<FRepState> RepState = RollbackActor.SubObjRepState.FindRef(ActorComp->GetFullName());
		FReceivingRepState* SubObjReceivingRepState = RepState.IsValid() ? RepState->GetReceivingRepState() : nullptr;

		if (SubObjReceivingRepState)
		{
			FRepObjectDataBuffer ActorCompData(ActorComp);
			FConstRepShadowDataBuffer ShadowData(SubObjReceivingRepState->StaticBuffer.GetData());

			SubObjLayout->DiffStableProperties(&SubObjReceivingRepState->RepNotifies, nullptr, ActorCompData, ShadowData);
		}
	}
}

void UDemoNetDriver::RespawnNecessaryNetStartupActors(TArray<AActor*>& SpawnedActors, ULevel* Level /* = nullptr */)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RespawnNecessaryNetStartupActors);

	TGuardValue<bool> RestoringStartupActors(bIsRestoringStartupActors, true);

	const FName FilterLevelName = Level ? Level->GetOutermost()->GetFName() : NAME_None;

	for (auto RollbackIt = RollbackNetStartupActors.CreateIterator(); RollbackIt; ++RollbackIt)
	{
		if (ReplayHelper.PlaybackDeletedNetStartupActors.Contains(RollbackIt.Key()))
		{
			// We don't want to re-create these since they should no longer exist after the current checkpoint
			continue;
		}

		FRollbackNetStartupActorInfo& RollbackActor = RollbackIt.Value();

		// filter to a specific level
		if ((Level != nullptr) && (RollbackActor.LevelName != FilterLevelName))
		{
			continue;
		}

		if (HasLevelStreamingFixes())
		{
			const FString LevelPackageName = UWorld::RemovePIEPrefix(RollbackActor.LevelName.ToString());

			// skip rollback actors in streamed out levels (pending gc)
			if (!ReplayHelper.LevelStatusesByName.Contains(LevelPackageName))
			{
				continue;
			}

			FReplayHelper::FLevelStatus& LevelStatus = ReplayHelper.GetLevelStatus(LevelPackageName);
			if (!LevelStatus.bIsReady)
			{
				continue;
			}
		}

		ULevel* RollbackActorLevel = ReplayHelper.WeakLevelsByName.FindRef(RollbackActor.LevelName).Get();

		if (!ensureMsgf(RollbackActorLevel, TEXT("RespawnNecessaryNetStartupActors: Rollback actor level is nullptr: %s"), *RollbackActor.Name.ToString()))
		{
			continue;
		}

		AActor* ExistingActor = FindObjectFast<AActor>(RollbackActorLevel, RollbackActor.Name);
		if (ExistingActor)
		{
			ensureMsgf((!IsValidChecked(ExistingActor) || ExistingActor->IsUnreachable()), TEXT("RespawnNecessaryNetStartupActors: Renaming rollback actor that wasn't destroyed: %s"), *GetFullNameSafe(ExistingActor));
			ExistingActor->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		}

		FActorSpawnParameters SpawnInfo;

		SpawnInfo.Template							= CastChecked<AActor>(RollbackActor.Archetype);
		SpawnInfo.SpawnCollisionHandlingOverride	= ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.bNoFail							= true;
		SpawnInfo.Name								= RollbackActor.Name;
		SpawnInfo.OverrideLevel						= RollbackActorLevel;
		SpawnInfo.bDeferConstruction				= true;

		const FTransform SpawnTransform = FTransform(RollbackActor.Rotation, RollbackActor.Location, RollbackActor.Scale3D);

		AActor* Actor = World->SpawnActor(RollbackActor.Archetype->GetClass(), &SpawnTransform, SpawnInfo);
		if (Actor)
		{
			if (!ensure( Actor->GetFullName() == RollbackIt.Key()))
			{
				UE_LOGF(LogDemo, Log, "RespawnNecessaryNetStartupActors: NetStartupRollbackActor name doesn't match original: %ls, %ls", *Actor->GetFullName(), *RollbackIt.Key());
			}

			bool bValidObjReferences = true;

			for (UObject* ObjRef : RollbackActor.ObjReferences)
			{
				if (!IsValid(ObjRef))
				{
					bValidObjReferences = false;
					UE_LOGF(LogDemo, Warning, "RespawnNecessaryNetStartupActors: Rollback actor reference was gc'd, skipping state restore: %ls", *GetFullNameSafe(Actor));
					break;
				}
			}

			TSharedPtr<FRepLayout> RepLayout = GetObjectClassRepLayout(Actor->GetClass());
			FReceivingRepState* ReceivingRepState = RollbackActor.RepState.IsValid() ? RollbackActor.RepState->GetReceivingRepState() : nullptr;

			// Restore saved actor state
			if (RepLayout.IsValid() && ReceivingRepState && bValidObjReferences)
			{
				const ENetRole SavedRole = Actor->GetLocalRole();

				FRepObjectDataBuffer ActorData(Actor);
				FConstRepShadowDataBuffer ShadowData(ReceivingRepState->StaticBuffer.GetData());

				RepLayout->DiffStableProperties(&ReceivingRepState->RepNotifies, nullptr, ActorData, ShadowData);

				Actor->SetRole(SavedRole);
			}

			check(Actor->GetRemoteRole() != ROLE_Authority);

			Actor->bNetStartup = true;

			if (Actor->GetLocalRole() == ROLE_Authority)
			{
				Actor->SwapRoles();
			}

			TSet<UActorComponent*> DiffedComponents;

			if (bValidObjReferences)
			{
				// Restore replicated component state for any objects that exist prior to construction (default subobjects)
				for (UActorComponent* ActorComp : Actor->GetComponents())
				{
					if (ActorComp)
					{
						RestoreComponentState(ActorComp, RollbackActor);

						DiffedComponents.Add(ActorComp);
					}
				}
			}
			
			// Update transforms based on restored state
			Actor->UpdateComponentTransforms();

			// Finish spawning/construction
			Actor->FinishSpawning(SpawnTransform, true);

			if (bValidObjReferences)
			{
				// Restore replicated component state of anything new (created during construction)
				for (UActorComponent* ActorComp : Actor->GetComponents())
				{
					// Could have been created by FinishSpawning (construction script)
					if (ActorComp && !DiffedComponents.Contains(ActorComp))
					{
						RestoreComponentState(ActorComp, RollbackActor);
					}
				}
			}

			// Update transforms based on restored state, and dirty render state
			Actor->UpdateComponentTransforms();
			Actor->MarkComponentsRenderStateDirty();

			// BeginPlay
			{
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(PostNetInit);
				QUICK_SCOPE_CYCLE_COUNTER(Stat_PostNetInit);
				LLM_SCOPE_BYNAME(TEXT("UObject/NetworkPostInit"));
				LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Actor->GetPackage(), ELLMTagSet::Assets);
				LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Actor->GetClass(), ELLMTagSet::AssetClasses);
				UE_TRACE_METADATA_SCOPE_ASSET(Actor, Actor->GetClass());
				Actor->PostNetInit();
			}

			// Call actor rep notifies
			if (RepLayout.IsValid() && ReceivingRepState)
			{
				if (ReceivingRepState->RepNotifies.Num() > 0)
				{
					RepLayout->CallRepNotifies(ReceivingRepState, Actor);

					Actor->PostRepNotifies();
				}
			}

			if (bValidObjReferences)
			{
				// Call component rep notifies
				for (UActorComponent* ActorComp : Actor->GetComponents())
				{
					if (ActorComp)
					{
						TSharedPtr<FRepLayout> SubObjLayout = GetObjectClassRepLayout(ActorComp->GetClass());
						if (SubObjLayout.IsValid())
						{
							TSharedPtr<FRepState> RepState = RollbackActor.SubObjRepState.FindRef(ActorComp->GetFullName());
							FReceivingRepState* SubObjReceivingRepState = RepState.IsValid() ? RepState->GetReceivingRepState() : nullptr;

							if (SubObjReceivingRepState && SubObjReceivingRepState->RepNotifies.Num() > 0)
							{
								SubObjLayout->CallRepNotifies(SubObjReceivingRepState, ActorComp);

								ActorComp->PostRepNotifies();
							}
						}
					}
				}
			}

			check(Actor->GetRemoteRole() == ROLE_Authority);

			SpawnedActors.Add(Actor);
		}

		RollbackIt.RemoveCurrent();
	}

	RollbackNetStartupActors.Compact();
}

void UDemoNetDriver::PrepFastForwardLevels()
{
	if (!HasLevelStreamingFixes() || ReplayHelper.NewStreamingLevelsThisFrame.Num() == 0)
	{
		return;
	}

	check(!bIsFastForwarding);
	check(!ReplayHelper.bIsLoadingCheckpoint);

	// Do a quick pass to double check everything is still valid, and that we have data for the levels.
	for (TWeakObjectPtr<UObject>& WeakLevel : ReplayHelper.NewStreamingLevelsThisFrame)
	{
		// For playback, we should only ever see ULevels in this list.
		if (ULevel* Level = CastChecked<ULevel>(WeakLevel.Get()))
		{
			if (!ensure(!ReplayHelper.LevelsPendingFastForward.Contains(Level)))
			{
				UE_LOGF(LogDemo, Warning, "FastForwardLevels - NewStreamingLevel found in Pending list! %ls", *GetFullName(Level));
				continue;
			}

			ReplayHelper.LevelsPendingFastForward.Add(Level);
		}
	}

	ReplayHelper.NewStreamingLevelsThisFrame.Empty();

	if (ReplayHelper.LevelsPendingFastForward.Num() == 0 ||
		LastProcessedPacketTime == 0.f ||
		// If there's already a FastForwardLevelsTask or GotoTimeTask, then we don't need
		// to add another (as the levels will get picked up by either of those).
		IsNamedTaskInQueue(ReplayTaskNames::GotoTimeInSecondsTask) ||
		IsNamedTaskInQueue(ReplayTaskNames::FastForwardLevelsTask))
	{
		return;
	}

	AddReplayTask(new FFastForwardLevelsTask(this));
}

bool UDemoNetDriver::ProcessFastForwardPackets(TArrayView<FPlaybackPacket> Packets, const TSet<int32>& LevelIndices)
{
	// Process all the packets we need.
	for (FPlaybackPacket& Packet : Packets)
	{
		// Skip packets that aren't associated with levels.
		if (Packet.SeenLevelIndex == 0)
		{
			UE_LOGF(LogDemo, Warning, "ProcessFastForwardPackets: Skipping packet with no seen level index");
			continue;
		}

		// Don't attempt to go beyond the current demo time.
		// These packets should have been already been filtered out while reading.
		if (!ensureMsgf(Packet.TimeSeconds <= GetDemoCurrentTime(), TEXT("UDemoNetDriver::FastForwardLevels: Read packet beyond DemoCurrentTime DemoTime = %f PacketTime = %f"), GetDemoCurrentTime(), Packet.TimeSeconds))
		{
			break;
		}

		if (ReplayHelper.SeenLevelStatuses.IsValidIndex(Packet.SeenLevelIndex - 1))
		{
			const FReplayHelper::FLevelStatus& LevelStatus = ReplayHelper.GetLevelStatus(Packet.SeenLevelIndex);
			const bool bCareAboutLevel = LevelIndices.Contains(LevelStatus.LevelIndex);

			if (bCareAboutLevel)
			{
				// If we tried to process the packet, but failed, then the replay will be in a broken state.
				// ProcessPacket will have called StopDemo.
				if (!ProcessPacket(Packet.Data.GetData(), Packet.Data.Num()))
				{
					UE_LOGF(LogDemo, Warning, "FastForwardLevel failed to process packet");
					return false;
				}
			}
		}
		else
		{
			UE_LOGF(LogDemo, Warning, "FastForwardLevel could not process packet with invalid seen level index");
		}
	}

	return true;
}

bool UDemoNetDriver::FastForwardLevels(const FGotoResult& GotoResult)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FastForwardLevels time"), STAT_FastForwardLevelTime, STATGROUP_Net);

	FArchive* CheckpointArchive = GetReplayStreamer()->GetCheckpointArchive();

	PauseChannels(false);

	// We can skip processing the checkpoint here, because Goto will load one up for us later.
	// We only want to check the very next task, though. Otherwise, we could end processing other
	// tasks in an invalid state.
	if (GetNextQueuedTaskName() == ReplayTaskNames::GotoTimeInSecondsTask)
	{
		// This is a bit hacky, but we don't want to do *any* processing this frame.
		// Therefore, we'll reset the ActiveReplayTask and return false.
		// This will cause us to early out, and then handle the Goto task next frame.
		ActiveReplayTask.Reset();
		return false;
	}

	// Generate the list of level names, and an uber list of the startup actors.
	// We manually track whenever a level is added and removed from the world, so these should always be valid.
	TSet<int32> LevelIndices;
	TSet<TWeakObjectPtr<AActor>> StartupActors;
	TSet<ULevel*> LocalLevels;

	// Reserve some default space, and just assume a minimum of at least 4 actors per level (super low estimate).
	LevelIndices.Reserve(ReplayHelper.LevelsPendingFastForward.Num());
	StartupActors.Reserve(ReplayHelper.LevelsPendingFastForward.Num() * 4);

	struct FLocalReadPacketsHelper
	{
		FLocalReadPacketsHelper(UDemoNetDriver& InDriver, const float InLastPacketTime):
			Driver(InDriver),
			LastPacketTime(InLastPacketTime)
		{
		}

		// @return True if another read can be attempted, false otherwise.
		bool ReadPackets(FArchive& Ar)
		{
			// Grab the packets, and make sure the stream is OK.
			PreFramePos = Ar.Tell();
			NumPackets = Packets.Num();
			if (!Driver.ReadDemoFrameIntoPlaybackPackets(Ar, Packets, true, &LastReadTime))
			{
				bErrorOccurred = true;
				return false;
			}

			// In case the archive had more data than we needed, we'll try to leave it where we left off
			// before the level fast forward.
			else if (LastReadTime > LastPacketTime)
			{
				Ar.Seek(PreFramePos);
				if (ensure(NumPackets != 0))
				{
					Packets.RemoveAt(NumPackets, Packets.Num() - NumPackets);
				}
				return false;
			}

			return true;
		}

		bool IsError() const
		{
			return bErrorOccurred;
		}

		TArray<FPlaybackPacket> Packets;

	private:

		UDemoNetDriver& Driver;
		const float LastPacketTime;

		// We only want to process packets that are before anything we've currently processed.
		// Further, we want to make sure that we leave the archive in a good state for later use.
		int32 NumPackets = 0;
		float LastReadTime = 0;
		FArchivePos PreFramePos = 0;

		bool bErrorOccurred = false;

	} ReadPacketsHelper(*this, LastProcessedPacketTime);

	ReplayHelper.PlaybackDeletedNetStartupActors.Empty();

	PlaybackDeltaCheckpointData.Empty();

	TArray<TInterval<int32>> DeltaCheckpointPacketIntervals;
	const bool bDeltaCheckpoint = HasDeltaCheckpoints();

	{
		auto IgnoreReceivedExportGUIDs = Cast<UPackageMapClient>(ServerConnection->PackageMap)->ScopedIgnoreReceivedExportGUIDs();

		// First, read in the checkpoint data (if any is available);
		if (CheckpointArchive->TotalSize() != 0)
		{
			ReplayHelper.SetPlaybackNetworkVersions(*CheckpointArchive);

			CheckpointArchive->ArMaxSerializeSize = FReplayHelper::MAX_DEMO_STRING_SERIALIZATION_SIZE;

			TGuardValue<bool> LoadingCheckpointGuard(ReplayHelper.bIsLoadingCheckpoint, true);

			FReplayCustomVersion::Type PlaybackReplayVersion = GetPlaybackReplayVersion();

			do 
			{
				FArchivePos MaxArchivePos = 0;

				if (bDeltaCheckpoint)
				{
					uint32 CheckpointSize = 0;
					*CheckpointArchive << CheckpointSize;

					MaxArchivePos = CheckpointArchive->Tell() + CheckpointSize;
				}

				TGuardValue<int64> MaxArchivePosGuard(MaxArchiveReadPos, MaxArchivePos);

				FArchivePos PacketOffset = 0;
				*CheckpointArchive << PacketOffset;

				PacketOffset += CheckpointArchive->Tell();

				int32 LevelIndex = INDEX_NONE;
				*CheckpointArchive << LevelIndex;

				if (bDeltaCheckpoint)
				{
					TUniquePtr<FDeltaCheckpointData>& CheckpointData = PlaybackDeltaCheckpointData.Emplace_GetRef(new FDeltaCheckpointData());

					ReplayHelper.ReadDeletedStartupActors(ServerConnection, *CheckpointArchive, CheckpointData->DestroyedNetStartupActors);
					ReplayHelper.PlaybackDeletedNetStartupActors.Append(CheckpointData->DestroyedNetStartupActors);

					*CheckpointArchive << CheckpointData->DestroyedDynamicActors;
					*CheckpointArchive << CheckpointData->ChannelsToClose;
				}
				else
				{
					ReplayHelper.PlaybackDeletedNetStartupActors.Empty();

					ReplayHelper.ReadDeletedStartupActors(ServerConnection, *CheckpointArchive, ReplayHelper.PlaybackDeletedNetStartupActors);
				}

				CheckpointArchive->Seek(PacketOffset);

				int32 DeltaPacketStartIndex = INDEX_NONE;
				if (bDeltaCheckpoint)
				{
					DeltaPacketStartIndex = ReadPacketsHelper.Packets.Num();
				}

				if (!ReadPacketsHelper.ReadPackets(*CheckpointArchive) && ReadPacketsHelper.IsError())
				{
					UE_LOGF(LogDemo, Warning, "UDemoNetDriver::FastForwardLevels: Failed to read packets from Checkpoint.");
					ReplayHelper.NotifyReplayError(EReplayResult::Unknown);
					return false;
				}

				if (bDeltaCheckpoint)
				{
					const int32 DeltaPacketEndIndex = ReadPacketsHelper.Packets.Num() - 1;
					if (DeltaPacketEndIndex >= DeltaPacketStartIndex)
					{
						DeltaCheckpointPacketIntervals.Emplace(DeltaPacketStartIndex, DeltaPacketEndIndex);
					}
				}
			} 
			while (!CheckpointArchive->IsError() && (CheckpointArchive->Tell() < CheckpointArchive->TotalSize()));
		}

		// Next, read in streaming data (if any is available)
		FArchive* StreamingAr = GetReplayStreamer()->GetStreamingArchive();
		check(StreamingAr);

		ReplayHelper.SetPlaybackNetworkVersions(*StreamingAr);

		int32 StreamPacketStartIndex = INDEX_NONE;
		if (bDeltaCheckpoint)
		{
			StreamPacketStartIndex = ReadPacketsHelper.Packets.Num();
		}

		while (!StreamingAr->AtEnd() && GetReplayStreamer()->IsDataAvailable() && ReadPacketsHelper.ReadPackets(*StreamingAr));

		if (ReadPacketsHelper.IsError())
		{
			UE_LOGF(LogDemo, Warning, "UDemoNetDriver::FastForwardLevels: Failed to read packets from Stream.");
			ReplayHelper.NotifyReplayError(EReplayResult::Serialization);
			return false;
		}

		if (bDeltaCheckpoint)
		{
			const int32 StreamPacketEndIndex = ReadPacketsHelper.Packets.Num() - 1;
			if (StreamPacketEndIndex >= StreamPacketStartIndex)
			{
				DeltaCheckpointPacketIntervals.Emplace(StreamPacketStartIndex, StreamPacketEndIndex);
			}
		}
	}

	// If we've gotten this far, it means we should have something to process.
	check(ReadPacketsHelper.Packets.Num() > 0);

	{
		TGuardValue<bool> RestoringStartupActors(bIsRestoringStartupActors, true);
		for (ULevel* Level : ReplayHelper.LevelsPendingFastForward)
		{
			// Track the appropriate level, and mark it as ready.
			FReplayHelper::FLevelStatus& LevelStatus = ReplayHelper.GetLevelStatus(ReplayHelper.GetLevelPackageName(*Level));
			LevelIndices.Add(LevelStatus.LevelIndex);
			LevelStatus.bIsReady = true;

			TSet<TWeakObjectPtr<AActor>> LevelActors;
			for (AActor* Actor : Level->Actors)
			{
				if (Actor == nullptr || !Actor->IsNetStartupActor())
				{
					continue;
				}
				else if (ReplayHelper.PlaybackDeletedNetStartupActors.Contains(Actor->GetFullName()))
				{
					// Put this actor on the rollback list so we can undelete it during future scrubbing,
					// then delete it.
					QueueNetStartupActorForRollbackViaDeletion(Actor);
					World->DestroyActor(Actor, true);
				}
				else
				{
					if (RollbackNetStartupActors.Contains(Actor->GetFullName()))
					{
						World->DestroyActor(Actor, true);
					}
					else
					{
						StartupActors.Add(Actor);
					}
				}
			}

			TArray<AActor*> SpawnedActors;
			RespawnNecessaryNetStartupActors(SpawnedActors, Level);

			for (AActor* Actor : SpawnedActors)
			{
				StartupActors.Add(Actor);
			}

			LocalLevels.Add(Level);
		}
	}

	ReplayHelper.LevelsPendingFastForward.Reset();

	{
		TGuardValue<bool> FastForward(bIsFastForwarding, true);
		FScopedAllowExistingChannelIndex ScopedAllowExistingChannelIndex(ServerConnection);

		if (bDeltaCheckpoint)
		{
			UDemoNetConnection* DemoConnection = CastChecked<UDemoNetConnection>(ServerConnection);

			for (int32 i = 0; i < DeltaCheckpointPacketIntervals.Num(); ++i)
			{
				if (PlaybackDeltaCheckpointData.IsValidIndex(i))
				{
					check(PlaybackDeltaCheckpointData[i].IsValid());

					for (auto& ChannelPair : PlaybackDeltaCheckpointData[i]->ChannelsToClose)
					{
						if (UActorChannel* ActorChannel = DemoConnection->GetOpenChannelMap().FindRef(ChannelPair.Key))
						{
							if (AActor* Actor = ActorChannel->GetActor())
							{
								if (LocalLevels.Contains(Actor->GetLevel()))
								{
									ActorChannel->ConditionalCleanUp(true, ChannelPair.Value);
								}
							}
						}
					}
				}

				check(DeltaCheckpointPacketIntervals[i].IsValid());
				check(ReadPacketsHelper.Packets.IsValidIndex(DeltaCheckpointPacketIntervals[i].Min));
				check(ReadPacketsHelper.Packets.IsValidIndex(DeltaCheckpointPacketIntervals[i].Min + DeltaCheckpointPacketIntervals[i].Size()));

				ProcessFastForwardPackets(MakeArrayView<FPlaybackPacket>(&ReadPacketsHelper.Packets[DeltaCheckpointPacketIntervals[i].Min], DeltaCheckpointPacketIntervals[i].Size() + 1), LevelIndices);
			}
		}
		else
		{
			ProcessFastForwardPackets(ReadPacketsHelper.Packets, LevelIndices);
		}
	}

	if (ensure(ServerConnection != nullptr))
	{
		// Make a pass at OnReps for startup actors, since they were skipped during checkpoint loading.
		// At this point the shadow state of these actors should be the actual state from before the checkpoint,
		// and the current state is the CDO state evolved by any changes that occurred during checkpoint loading and fast-forwarding.

		TArray<UActorChannel*> ChannelsToUpdate;
		ChannelsToUpdate.Reserve(StartupActors.Num());

		for (UChannel* Channel : ServerConnection->OpenChannels)
		{
			// Skip non-actor channels.
			if (Channel == nullptr || Channel->ChName != NAME_Actor)
			{
				continue;
			}

			// Since we know this is an actor channel, should be safe to do a static_cast.
			UActorChannel* const ActorChannel = static_cast<UActorChannel*>(Channel);
			if (AActor* Actor = ActorChannel->GetActor())
			{
				const bool bDynamicInLevel = !Actor->IsNetStartupActor() && LocalLevels.Contains(Actor->GetLevel());

				// We only need to consider startup actors, or dynamic that were spawned and outered
				// to one of our sublevels.
				if (bDynamicInLevel || StartupActors.Contains(Actor))
				{
					ChannelsToUpdate.Add(ActorChannel);

					DiffActorProperties(ActorChannel);
				}
			}
		}

		for (UActorChannel* Channel : ChannelsToUpdate)
		{
			for (auto& ReplicatorPair : Channel->ReplicationMap)
			{
				ReplicatorPair.Value->CallRepNotifies(true);
			}
		}

		auto CallRepNotifies = [](FObjectKey OwnerActorKey, FObjectKey ObjectKey, const TSharedRef<FObjectReplicator>& ReplicatorRef)
		{
			ReplicatorRef->CallRepNotifies(true);
		};

		ServerConnection->ExecuteOnAllDormantReplicators(CallRepNotifies);
	}

	return true;
}

bool UDemoNetDriver::LoadCheckpoint(const FGotoResult& GotoResult)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("LoadCheckpoint time"), STAT_ReplayCheckpointLoadTime, STATGROUP_Net);

	FArchive* GotoCheckpointArchive = GetReplayStreamer()->GetCheckpointArchive();

	check(GotoCheckpointArchive != nullptr);
	check(!bIsFastForwardingForCheckpoint);
	check(!bIsFastForwarding);

	ReplayHelper.SetPlaybackNetworkVersions(*GotoCheckpointArchive);

	GotoCheckpointArchive->ArMaxSerializeSize = FReplayHelper::MAX_DEMO_STRING_SERIALIZATION_SIZE;

	int32 LevelForCheckpoint = 0;

	const bool bDeltaCheckpoint = HasDeltaCheckpoints();

	if (bDeltaCheckpoint)
	{
		if (GotoCheckpointArchive->TotalSize() > 0)
		{
			uint32 CheckpointSize = 0;
			*GotoCheckpointArchive << CheckpointSize;
		}
	}

	if (HasLevelStreamingFixes())
	{
		// Make sure to read the packet offset, even though we won't use it here.
		if (GotoCheckpointArchive->TotalSize() > 0)
		{
			FArchivePos PacketOffset = 0;
			*GotoCheckpointArchive << PacketOffset;
		}

		ReplayHelper.ResetLevelStatuses();
	}

	ReplayHelper.ResetLevelMap();

	LastProcessedPacketTime = 0.f;
	ReplayHelper.LatestReadFrameTime = 0.f;

	FReplayCustomVersion::Type PlaybackReplayVersion = GetPlaybackReplayVersion();

	if (GotoCheckpointArchive->TotalSize() > 0)
	{
		*GotoCheckpointArchive << LevelForCheckpoint;
	}

	check(World);

	if (LevelForCheckpoint != GetCurrentLevelIndex())
	{
		World->GetGameInstance()->OnSeamlessTravelDuringReplay();

		for (FActorIterator ActorIt(World); ActorIt; ++ActorIt)
		{
			World->DestroyActor(*ActorIt, true);
		}

		// Clean package map to prepare to restore it to the checkpoint state
		GetNetGuidCache()->ResetCacheForDemo();

		// Since we only count the number of sub-spectators, add one more slot for main spectator
		// Very small optimization. We do want to clear this so that we don't end up doing during ProcessSeamlessTravel
		SpectatorControllers.Empty(CleanUpSplitscreenConnections(true) + 1);
		SpectatorController = nullptr;

		ServerConnection->Close();
		ServerConnection->CleanUp();

		// Recreate the server connection - this is done so that when we execute the code the below again when we read in the
		// checkpoint again after the server travel is finished, we'll have a clean server connection to work with.
		ServerConnection = NewObject<UNetConnection>(GetTransientPackage(), UDemoNetConnection::StaticClass());

		FURL ConnectURL;
		ConnectURL.Map = ReplayHelper.DemoURL.Map;
		ServerConnection->InitConnection(this, USOCK_Pending, ConnectURL, 1000000);

		GEngine->ForceGarbageCollection(true);

		ProcessSeamlessTravel(LevelForCheckpoint);
		SetCurrentLevelIndex(LevelForCheckpoint);

		if (GotoCheckpointArchive->TotalSize() != 0 && GotoCheckpointArchive->TotalSize() != INDEX_NONE)
		{
			GotoCheckpointArchive->Seek(0);
		}

		return false;
	}

	// Save off the current spectator position
	// Check for nullptr, which can be the case if we haven't played any of the demo yet but want to fast forward (joining live game for example)
	if (SpectatorController != nullptr)
	{
		// Save off the SpectatorController's GUID so that we know not to queue its bunches
		AddNonQueuedActorForScrubbing(SpectatorController);
	}

	// Remember the spectator controller's view target so we can restore it
	FNetworkGUID ViewTargetGUID;

	if (SpectatorController && SpectatorController->GetViewTarget())
	{
		ViewTargetGUID = GetNetGuidCache()->NetGUIDLookup.FindRef(SpectatorController->GetViewTarget());

		if (ViewTargetGUID.IsValid())
		{
			AddNonQueuedActorForScrubbing(SpectatorController->GetViewTarget());
		}
	}

	PauseChannels(false);

	FNetworkReplayDelegates::OnPreScrub.Broadcast(World);

	ReplayHelper.bIsLoadingCheckpoint = true;

	struct FPreservedNetworkGUIDEntry
	{
		FPreservedNetworkGUIDEntry(const FNetworkGUID InNetGUID, const AActor* const InActor)
			: NetGUID(InNetGUID), Actor(InActor) {}

		FNetworkGUID NetGUID;
		const AActor* Actor;
	};

	// Store GUIDs for the spectator controller and any of its owned actors, so we can find them when we process the checkpoint.
	// For the spectator controller, this allows the state and position to persist.
	TArray<FPreservedNetworkGUIDEntry> NetGUIDsToPreserve;

	if (!ensureMsgf(TrackedRewindActorsByGUID.Num() == 0, TEXT("LoadCheckpoint: TrackedRewindAcotrsByGUID list not empty!")))
	{
		TrackedRewindActorsByGUID.Empty();
	}

	TSet<const AActor*> KeepAliveActors;

	// Determine if an Actor has a reference to a spectator in some way.
	// This prevents garbage collection on splitscreen playercontrollers
	auto HasPlayerSpectatorRef = [this](const AActor* InActor) -> bool 
	{
		for (const APlayerController* CurSpectator : SpectatorControllers)
		{
			if (IsValid(CurSpectator) &&
				(InActor == CurSpectator || InActor == CurSpectator->GetSpectatorPawn()	|| InActor->IsOwnedBy(CurSpectator)))
			{
				return true;
			}
		}
		return false;
	};


	// Destroy all non startup actors. They will get restored with the checkpoint
	for (FActorIterator ActorIt(World); ActorIt; ++ActorIt)
	{
		AActor* CurrentActor = *ActorIt;

		// If there are any existing actors that are bAlwaysRelevant, don't queue their bunches.
		// Actors that do queue their bunches might not appear immediately after the checkpoint is loaded,
		// and missing bAlwaysRelevant actors are more likely to cause noticeable artifacts.
		// NOTE - We are adding the actor guid here, under the assumption that the actor will reclaim the same guid when we load the checkpoint
		// This is normally the case, but could break if actors get destroyed and re-created with different guids during recording
		if (CurrentActor->bAlwaysRelevant)
		{
			AddNonQueuedActorForScrubbing(CurrentActor);
		}
		
		const bool bShouldPreserveForPlayerController = HasPlayerSpectatorRef(CurrentActor);
		const bool bShouldPreserveForRewindability = (CurrentActor->bReplayRewindable && !CurrentActor->IsNetStartupActor());

		if (bShouldPreserveForPlayerController || bShouldPreserveForRewindability)
		{
			// If an non-startup actor that we don't destroy has an entry in the GuidCache, preserve that entry so
			// that the object will be re-used after loading the checkpoint. Otherwise, a new copy
			// of the object will be created each time a checkpoint is loaded, causing a leak.
			const FNetworkGUID FoundGUID = GetNetGuidCache()->NetGUIDLookup.FindRef(CurrentActor);
				
			if (FoundGUID.IsValid())
			{
				NetGUIDsToPreserve.Emplace(FoundGUID, CurrentActor);
				
				if (bShouldPreserveForRewindability)
				{
					TrackedRewindActorsByGUID.Add(FoundGUID);
				}
			}

			KeepAliveActors.Add(CurrentActor);
			continue;
		}

		// Prevent NetStartupActors from being destroyed.
		// NetStartupActors that can't have properties directly re-applied should use QueueNetStartupActorForRollbackViaDeletion.
		if (CurrentActor->IsNetStartupActor())
		{
			// Go ahead and rewind this now, since we won't be destroying it later.
			if (CurrentActor->bReplayRewindable)
			{
				CurrentActor->RewindForReplay();
			}
			KeepAliveActors.Add(CurrentActor);
			continue;
		}

		World->DestroyActor(CurrentActor, true);
	}

	// Destroy all particle FX attached to the WorldSettings (the WorldSettings actor persists but the particle FX spawned at runtime shouldn't)
	World->HandleTimelineScrubbed();

	// Remove references to our KeepAlive actors so that cleaning up the channels won't destroy them.
	for (int32 i = ServerConnection->OpenChannels.Num() - 1; i >= 0; i--)
	{
		UChannel* OpenChannel = ServerConnection->OpenChannels[i];
		if ( OpenChannel != nullptr )
		{
			UActorChannel* ActorChannel = Cast<UActorChannel>(OpenChannel);
			if (ActorChannel != nullptr && KeepAliveActors.Contains(ActorChannel->Actor))
			{
				ActorChannel->ReleaseReferences(false);
			}
		}
	}

	if (ServerConnection->OwningActor == SpectatorController)
	{
		ServerConnection->OwningActor = nullptr;
	}

	ReplayHelper.ExternalDataToObjectMap.Empty();

	PlaybackPackets.Empty();
	ReplayHelper.PlaybackFrames.Empty();

	{
		TGuardValue<bool> RestoringStartupActors(bIsRestoringStartupActors, true);

		// Destroy startup actors that need to rollback via being destroyed and re-created
		for (FActorIterator ActorIt(World); ActorIt; ++ActorIt)
		{
			if (RollbackNetStartupActors.Contains(ActorIt->GetFullName()))
			{
				World->DestroyActor(*ActorIt, true);
			}
		}
	}

	// Going to be recreating the splitscreen connections, but keep around the player controller.
	CleanUpSplitscreenConnections(false);
	ServerConnection->Close();
	ServerConnection->CleanUp();

	FNetworkReplayDelegates::OnScrubTeardown.Broadcast(World);

	// Optionally collect garbage after the old actors and connection are cleaned up - there could be a lot of pending-kill objects at this point.
	if (CVarDemoLoadCheckpointGarbageCollect.GetValueOnGameThread() != 0)
	{
		GEngine->ForceGarbageCollection(true);
	}

	FURL ConnectURL;
	ConnectURL.Map = ReplayHelper.DemoURL.Map;

	ServerConnection = NewObject<UNetConnection>(GetTransientPackage(), UDemoNetConnection::StaticClass());
	ServerConnection->InitConnection( this, USOCK_Pending, ConnectURL, 1000000 );

	// Set network version on connection
	ReplayHelper.SetPlaybackNetworkVersions(ServerConnection);

	// Create fake control channel
	CreateInitialClientChannels();

	// Respawn child connections as the parent connection has been recreated.
	for (APlayerController* CurController : SpectatorControllers)
	{
		if (CurController != SpectatorController)
		{
			RestoreConnectionPostScrub(CurController, CreateChild(ServerConnection));
		}
	}

	// Catch a rare case where the spectator controller is null, but a valid GUID is
	// found on the GuidCache. The weak pointers in the NetGUIDLookup map are probably
	// going null, and we want catch these cases and investigate further.
	if (!ensure(GetNetGuidCache()->NetGUIDLookup.FindRef(SpectatorController).IsValid() == (SpectatorController != nullptr)))
	{
		UE_LOGF(LogDemo, Log, "LoadCheckpoint: SpectatorController is null and a valid GUID for null was found in the GuidCache. SpectatorController = %ls",
			*GetFullNameSafe(SpectatorController));
	}

	// Clean package map to prepare to restore it to the checkpoint state
	FlushAsyncLoading();
	GetNetGuidCache()->ResetCacheForDemo();

	// Restore preserved packagemap entries
	for (const FPreservedNetworkGUIDEntry& PreservedEntry : NetGUIDsToPreserve)
	{
		check(PreservedEntry.NetGUID.IsValid());
		
		FNetGuidCacheObject& CacheObject = GetNetGuidCache()->ObjectLookup.FindOrAdd(PreservedEntry.NetGUID);

		CacheObject.Object = MakeWeakObjectPtr(const_cast<AActor*>(PreservedEntry.Actor));
		check(CacheObject.Object != nullptr);
		CacheObject.bNoLoad = true;
		GetNetGuidCache()->NetGUIDLookup.Add(CacheObject.Object, PreservedEntry.NetGUID);
	}

	if (GotoCheckpointArchive->TotalSize() == 0 || GotoCheckpointArchive->TotalSize() == INDEX_NONE)
	{
		// Make sure this is empty so that RespawnNecessaryNetStartupActors will respawn them
		ReplayHelper.PlaybackDeletedNetStartupActors.Empty();

		// Re-create all startup actors that were destroyed but should exist beyond this point
		TArray<AActor*> SpawnedActors;
		RespawnNecessaryNetStartupActors(SpawnedActors);

		// This is the very first checkpoint, we'll read the stream from the very beginning in this case
		SetDemoCurrentTime(0.0f);
		ReplayHelper.bIsLoadingCheckpoint	= false;

		if (GotoResult.ExtraTimeMS != -1)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SkipTimeInternal((float)GotoResult.ExtraTimeMS / 1000.0f, true, true);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			// Make sure that we delete any Rewind actors that aren't valid anymore.
			// If there's more data to stream in, we will handle this in FinalizeFastForward.
			CleanupOutstandingRewindActors();
		}

		return true;
	}

	GotoCheckpointArchive->Seek(0);

	ReplayHelper.PlaybackDeletedNetStartupActors.Empty();

	PlaybackDeltaCheckpointData.Empty();

	TArray<TInterval<int32>> DeltaCheckpointPacketIntervals;
	TArray<FName> PathNameTable;

	do
	{
		FArchivePos MaxArchivePos = 0;

		if (bDeltaCheckpoint)
		{
			uint32 CheckpointSize = 0;
			*GotoCheckpointArchive << CheckpointSize;

			MaxArchivePos = GotoCheckpointArchive->Tell() + CheckpointSize;
		}

		TGuardValue<int64> MaxArchivePosGuard(MaxArchiveReadPos, MaxArchivePos);
			
		if (HasLevelStreamingFixes())
		{
			FArchivePos PacketOffset = 0;
			*GotoCheckpointArchive << PacketOffset;
		}

		int32 LevelIndex = INDEX_NONE;
		*GotoCheckpointArchive << LevelIndex;

		// Load net startup actors that need to be destroyed
		if (bDeltaCheckpoint)
		{
			TUniquePtr<FDeltaCheckpointData>& CheckpointData = PlaybackDeltaCheckpointData.Emplace_GetRef(new FDeltaCheckpointData());

			ReplayHelper.ReadDeletedStartupActors(ServerConnection, *GotoCheckpointArchive, CheckpointData->DestroyedNetStartupActors);

			ReplayHelper.PlaybackDeletedNetStartupActors.Append(CheckpointData->DestroyedNetStartupActors);

			*GotoCheckpointArchive << CheckpointData->DestroyedDynamicActors;
			*GotoCheckpointArchive << CheckpointData->ChannelsToClose;
		}
		else
		{
			ReplayHelper.PlaybackDeletedNetStartupActors.Empty();

			ReplayHelper.ReadDeletedStartupActors(ServerConnection, *GotoCheckpointArchive, ReplayHelper.PlaybackDeletedNetStartupActors);
		}

		int32 NumValues = 0;
		*GotoCheckpointArchive << NumValues;

		for (int32 i = 0; i < NumValues; i++)
		{
			FNetworkGUID Guid;

			*GotoCheckpointArchive << Guid;

			FNetGuidCacheObject CacheObject;

			*GotoCheckpointArchive << CacheObject.OuterGUID;

			FString PathName;

			if (PlaybackReplayVersion < FReplayCustomVersion::GuidNameTable)
			{
				*GotoCheckpointArchive << PathName;
			}
			else
			{
				uint8 bExported = 0;
				*GotoCheckpointArchive << bExported;

				if (bExported == 1)
				{
					*GotoCheckpointArchive << PathName;

					PathNameTable.Add(FName(*PathName));
				}
				else
				{
					uint32 PathNameIndex = 0;
					GotoCheckpointArchive->SerializeIntPacked(PathNameIndex);

					if (PathNameTable.IsValidIndex(PathNameIndex))
					{
						PathName = PathNameTable[PathNameIndex].ToString();
					}
					else
					{
						GotoCheckpointArchive->SetError();
						UE_LOGF(LogDemo, Error, "Invalid guid path table index while serializing checkpoint.");
						break;
					}
				}
			}

			// Remap the pathname to handle client-recorded replays
			GEngine->NetworkRemapPath(ServerConnection, PathName, true);

			CacheObject.PathName = FName(*PathName);

			if (PlaybackReplayVersion < FReplayCustomVersion::GuidCacheChecksums)
			{
				*GotoCheckpointArchive << CacheObject.NetworkChecksum;
			}

			uint8 Flags = 0;
			*GotoCheckpointArchive << Flags;

			CacheObject.bNoLoad = (Flags & (1 << 0)) ? true : false;
			CacheObject.bIgnoreWhenMissing = (Flags & (1 << 1)) ? true : false;		

			GetNetGuidCache()->ObjectLookup.Add(Guid, CacheObject);

			if (GotoCheckpointArchive->IsError())
			{
				UE_LOGF(LogDemo, Error, "Guid cache serialization error while loading checkpoint.");
				return true;
			}
		}

		int32 DeltaPacketStartIndex = INDEX_NONE;

		// Read in the compatible rep layouts in this checkpoint
		if (bDeltaCheckpoint)
		{
			CastChecked<UPackageMapClient>(ServerConnection->PackageMap)->SerializeNetFieldExportDelta(*GotoCheckpointArchive);

			DeltaPacketStartIndex = PlaybackPackets.Num();
		}
		else
		{
			CastChecked<UPackageMapClient>(ServerConnection->PackageMap)->SerializeNetFieldExportGroupMap(*GotoCheckpointArchive);
		}

		if (bDeltaCheckpoint)
		{
			// each set of checkpoint packets we read will have a full name table, so only keep the last version
			ReplayHelper.SeenLevelStatuses.Reset();
		}

		ReadDemoFrameIntoPlaybackPackets(*GotoCheckpointArchive);

		if (bDeltaCheckpoint)
		{
			const int32 DeltaPacketEndIndex = PlaybackPackets.Num() - 1;
			if (DeltaPacketEndIndex >= DeltaPacketStartIndex)
			{
				DeltaCheckpointPacketIntervals.Emplace(DeltaPacketStartIndex, DeltaPacketEndIndex);
			}
		}
	}
	while (!GotoCheckpointArchive->IsError() && (GotoCheckpointArchive->Tell() < GotoCheckpointArchive->TotalSize()));
	
	if (World != nullptr)
	{
		TGuardValue<bool> RestoringStartupActors(bIsRestoringStartupActors, true);
		// Destroy startup actors that shouldn't exist past this checkpoint
		for (FActorIterator ActorIt( World ); ActorIt; ++ActorIt)
		{
			AActor* CurrentActor = *ActorIt;

			const FString FullName = CurrentActor->GetFullName();

			if (ReplayHelper.PlaybackDeletedNetStartupActors.Contains(FullName))
			{
				if (CurrentActor->bReplayRewindable)
				{
					// Log and skip. We can't queue Rewindable actors and we can't destroy them.
					// This actor may still get destroyed during cleanup.
					UE_LOGF(LogDemo, Warning, "Replay Rewindable Actor found in the DeletedNetStartupActors. Replay may show artifacts (%ls)", *FullName);
					continue;
				}

				// Put this actor on the rollback list so we can undelete it during future scrubbing
				QueueNetStartupActorForRollbackViaDeletion(CurrentActor);

				UE_LOGF(LogDemo, Verbose, "LoadCheckpoint: deleting startup actor %ls", *FullName);

				// Delete the actor
				World->DestroyActor(CurrentActor, true);
			}
		}

		// Re-create all startup actors that were destroyed but should exist beyond this point
		TArray<AActor*> SpawnedActors;
		RespawnNecessaryNetStartupActors(SpawnedActors);
	}
		
	SetDemoCurrentTime((PlaybackPackets.Num() > 0) ? PlaybackPackets.Last().TimeSeconds : 0.0f);

	if (GotoResult.ExtraTimeMS != -1)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// If we need to skip more time for fine scrubbing, set that up now
		SkipTimeInternal((float)GotoResult.ExtraTimeMS / 1000.0f, true, true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		// Make sure that we delete any Rewind actors that aren't valid anymore.
		// If there's more data to stream in, we will handle this in FinalizeFastForward.
		CleanupOutstandingRewindActors();
	}

	{
		if (bDeltaCheckpoint)
		{
			UDemoNetConnection* DemoConnection = Cast<UDemoNetConnection>(ServerConnection);

			for (int32 i = 0; i < DeltaCheckpointPacketIntervals.Num(); ++i)
			{
				if (DemoConnection && PlaybackDeltaCheckpointData.IsValidIndex(i))
				{
					check(PlaybackDeltaCheckpointData[i].IsValid());

					for (auto& ChannelPair : PlaybackDeltaCheckpointData[i]->ChannelsToClose)
					{
						if (UActorChannel* ActorChannel = DemoConnection->GetOpenChannelMap().FindRef(ChannelPair.Key))
						{
							ActorChannel->ConditionalCleanUp(true, ChannelPair.Value);
						}
					}
				}

				check(DeltaCheckpointPacketIntervals[i].IsValid());
				check(PlaybackPackets.IsValidIndex(DeltaCheckpointPacketIntervals[i].Min)); 
				check(PlaybackPackets.IsValidIndex(DeltaCheckpointPacketIntervals[i].Min + DeltaCheckpointPacketIntervals[i].Size()));

				// + 1 because the interval is inclusive
				ProcessPlaybackPackets(MakeArrayView<FPlaybackPacket>(&PlaybackPackets[DeltaCheckpointPacketIntervals[i].Min], DeltaCheckpointPacketIntervals[i].Size() + 1));
			}

			PlaybackPackets.Empty();
			ReplayHelper.PlaybackFrames.Empty();
		}
		else
		{
			ProcessAllPlaybackPackets();
		}
	}

	ReplayHelper.bIsLoadingCheckpoint = false;

	// Save the replicated server time here
	if (World != nullptr)
	{
		const AGameStateBase* const GameState = World->GetGameState();
		if (GameState != nullptr)
		{
			SavedReplicatedWorldTimeSeconds = GameState->ReplicatedWorldTimeSecondsDouble;
		}
	}

	if (SpectatorController && ViewTargetGUID.IsValid())
	{
		AActor* ViewTarget = Cast<AActor>(GetNetGuidCache()->GetObjectFromNetGUID(ViewTargetGUID, false));

		if (ViewTarget)
		{
			SpectatorController->SetViewTarget(ViewTarget);
		}
	}

	return true;
}

bool UDemoNetDriver::IsSavingCheckpoint() const
{
	if (ClientConnections.Num() > 0)
	{
		UNetConnection* const NetConnection = ClientConnections[0];
		if (NetConnection)
		{
			return (NetConnection->ResendAllDataState != EResendAllDataState::None);
		}
	}

	return false;
}

bool UDemoNetDriver::ShouldQueueBunchesForActorGUID(FNetworkGUID InGUID) const
{
	if (CVarDemoQueueCheckpointChannels.GetValueOnGameThread() == 0)
	{
		return false;
	}

	// While loading a checkpoint, queue most bunches so that we don't process them all on one frame.
	if (bIsFastForwardingForCheckpoint)
	{
		return !NonQueuedGUIDsForScrubbing.Contains(InGUID);
	}

	return false;
}

bool UDemoNetDriver::ShouldIgnoreRPCs() const
{
	return (CVarDemoFastForwardIgnoreRPCs.GetValueOnAnyThread() && (ReplayHelper.bIsLoadingCheckpoint || bIsFastForwarding));
}

FNetworkGUID UDemoNetDriver::GetGUIDForActor(const AActor* InActor) const
{
	UNetConnection* Connection = ServerConnection;
	
	if (ClientConnections.Num() > 0)
	{
		Connection = ClientConnections[0];
	}

	if (!Connection)
	{
		return FNetworkGUID();
	}

	FNetworkGUID Guid = Connection->PackageMap->GetNetGUIDFromObject(InActor);
	return Guid;
}

AActor* UDemoNetDriver::GetActorForGUID(FNetworkGUID InGUID) const
{
	UNetConnection* Connection = ServerConnection;
	
	if (ClientConnections.Num() > 0)
	{
		Connection = ClientConnections[0];
	}

	if (!Connection)
	{
		return nullptr;
	}

	UObject* FoundObject = Connection->PackageMap->GetObjectFromNetGUID(InGUID, true);
	return Cast<AActor>(FoundObject);

}

bool UDemoNetDriver::ShouldReceiveRepNotifiesForObject(UObject* Object) const
{
	// Return false for startup actors during checkpoint loading, since they are
	// not destroyed and re-created like dynamic actors. Startup actors will
	// have their properties diffed and RepNotifies called after the checkpoint is loaded.

	if (!ReplayHelper.bIsLoadingCheckpoint && !bIsFastForwardingForCheckpoint)
	{
		return true;
	}

	const AActor* const Actor = Cast<AActor>(Object);
	const bool bIsStartupActor = Actor != nullptr && Actor->IsNetStartupActor();

	return !bIsStartupActor;
}

void UDemoNetDriver::AddNonQueuedActorForScrubbing(AActor const* Actor)
{
	UActorChannel const* const* const FoundChannel = ServerConnection->FindActorChannel(MakeWeakObjectPtr(const_cast<AActor*>(Actor)));
	if (FoundChannel != nullptr && *FoundChannel != nullptr)
	{
		FNetworkGUID const ActorGUID = (*FoundChannel)->ActorNetGUID;
		NonQueuedGUIDsForScrubbing.Add(ActorGUID);
	}
}

void UDemoNetDriver::AddNonQueuedGUIDForScrubbing(FNetworkGUID InGUID)
{
	if (InGUID.IsValid())
	{
		NonQueuedGUIDsForScrubbing.Add(InGUID);
	}
}

FDemoSavedRepObjectState::FDemoSavedRepObjectState(
	const TWeakObjectPtr<const UObject>& InObject,
	const TSharedRef<const FRepLayout>& InRepLayout,
	FRepStateStaticBuffer&& InPropertyData) :

	Object(InObject),
	RepLayout(InRepLayout),
	PropertyData(MoveTemp(InPropertyData))
{
}

FDemoSavedRepObjectState::~FDemoSavedRepObjectState()
{
}

FDemoSavedPropertyState UDemoNetDriver::SavePropertyState() const
{
	FDemoSavedPropertyState State;

	if (IsRecording())
	{
		const UNetConnection* const RecordingConnection = ClientConnections[0];
		for (auto ChannelPair = RecordingConnection->ActorChannelConstIterator(); ChannelPair; ++ChannelPair)
		{
			const UActorChannel* const Channel = ChannelPair.Value();
			if (Channel)
			{
				for (const auto& ReplicatorPair : Channel->ReplicationMap)
				{
					TWeakObjectPtr<UObject> WeakObjectPtr = ReplicatorPair.Value->GetWeakObjectPtr();
					if (const UObject* const RepObject = WeakObjectPtr.Get())
					{
						const TSharedRef<const FRepLayout> RepLayout = ReplicatorPair.Value->RepLayout.ToSharedRef();
						FDemoSavedRepObjectState& SavedObject = State.Emplace_GetRef(WeakObjectPtr, RepLayout, RepLayout->CreateShadowBuffer((const uint8*)RepObject));

						// TODO: InitShadowData should copy property data, so this seem unnecessary.
						// Store the properties in the new RepState
						FRepShadowDataBuffer ShadowData(SavedObject.PropertyData.GetData());
						FConstRepObjectDataBuffer RepObjectData(RepObject);

						SavedObject.RepLayout->DiffProperties(nullptr, ShadowData, RepObjectData, EDiffPropertiesFlags::Sync | EDiffPropertiesFlags::IncludeConditionalProperties);
					}
				}
			}
		}
	}

	return State;
}

bool UDemoNetDriver::ComparePropertyState(const FDemoSavedPropertyState& State) const
{
	bool bWasDifferent = false;

	if (IsRecording())
	{
		for (const FDemoSavedRepObjectState& ObjectState : State)
		{
			const UObject* const RepObject = ObjectState.Object.Get();
			if (RepObject)
			{
				FRepObjectDataBuffer RepObjectData(const_cast<UObject* const>(RepObject));
				FConstRepShadowDataBuffer ShadowData(ObjectState.PropertyData.GetData());

				if (ObjectState.RepLayout->DiffProperties(nullptr, RepObjectData, ShadowData, EDiffPropertiesFlags::IncludeConditionalProperties))
				{
					bWasDifferent = true;
				}
			}
			else
			{
				UE_LOGF(LogDemo, Warning, "A replicated object was destroyed or marked pending kill since its state was saved!");
				bWasDifferent = true;
			}
		}
	}

	return bWasDifferent;
}

void UDemoNetDriver::RestoreConnectionPostScrub(APlayerController* PC, UNetConnection* NetConnection)
{
	check(NetConnection != nullptr);
	check(PC != nullptr);

	PC->SetRole(ROLE_AutonomousProxy);
	PC->NetConnection = NetConnection;
	NetConnection->LastReceiveTime = GetElapsedTime();
	NetConnection->LastReceiveRealtime = FPlatformTime::Seconds();
	NetConnection->LastGoodPacketRealtime = FPlatformTime::Seconds();
	NetConnection->SetConnectionState(USOCK_Open);
	NetConnection->PlayerController = PC;
	NetConnection->OwningActor = PC;
}

void UDemoNetDriver::SetSpectatorController(APlayerController* PC)
{
	SpectatorController = PC;
	if (PC != nullptr)
	{
		SpectatorControllers.AddUnique(PC);
	}
}

TSharedPtr<FInternetAddr> FInternetAddrDemo::DemoInternetAddr = MakeShareable(new FInternetAddrDemo);

/*-----------------------------------------------------------------------------
	UDemoNetConnection.
-----------------------------------------------------------------------------*/

UDemoNetConnection::UDemoNetConnection( const FObjectInitializer& ObjectInitializer ) : Super( ObjectInitializer )
{
	MaxPacket = FReplayHelper::MAX_DEMO_READ_WRITE_BUFFER;
	SetInternalAck(true);
	SetReplay(true);
	SetAutoFlush(true);
	SetUnlimitedBunchSizeAllowed(true);
}

void UDemoNetConnection::InitConnection( UNetDriver* InDriver, EConnectionState InState, const FURL& InURL, int32 InConnectionSpeed, int32 InMaxPacket)
{
	// default implementation
	Super::InitConnection( InDriver, InState, InURL, InConnectionSpeed );

	MaxPacket = (InMaxPacket == 0 || InMaxPacket > FReplayHelper::MAX_DEMO_READ_WRITE_BUFFER) ? FReplayHelper::MAX_DEMO_READ_WRITE_BUFFER : InMaxPacket;
	SetInternalAck(true);
	SetReplay(true);
	SetAutoFlush(true);
	SetUnlimitedBunchSizeAllowed(true);

	InitSendBuffer();

	// the driver must be a DemoRecording driver (GetDriver makes assumptions to avoid Cast'ing each time)
	check( InDriver->IsA( UDemoNetDriver::StaticClass() ) );
}

FString UDemoNetConnection::LowLevelGetRemoteAddress( bool bAppendPort )
{
	return TEXT("UDemoNetConnection");
}

void UDemoNetConnection::LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits)
{
	uint32 CountBytes = FMath::DivideAndRoundUp(CountBits, 8);

	if (CountBytes == 0)
	{
		UE_LOGF(LogDemo, Warning, "UDemoNetConnection::LowLevelSend: Ignoring empty packet.");
		return;
	}

	UDemoNetDriver* DemoDriver = GetDriver();
	if (!DemoDriver)
	{
		UE_LOGF(LogDemo, Warning, "UDemoNetConnection::LowLevelSend: No driver found.");
		return;
	}

	if (CountBytes > FReplayHelper::MAX_DEMO_READ_WRITE_BUFFER)
	{
		UE_LOGF(LogDemo, Fatal, "UDemoNetConnection::LowLevelSend: CountBytes > MAX_DEMO_READ_WRITE_BUFFER.");
	}

	TrackSendForProfiler(Data, CountBytes);

	TArray<FQueuedDemoPacket>& QueuedPackets = (ResendAllDataState != EResendAllDataState::None) ? DemoDriver->ReplayHelper.QueuedCheckpointPackets : DemoDriver->ReplayHelper.QueuedDemoPackets;

	int32 NewIndex = QueuedPackets.Emplace((uint8*)Data, CountBits, Traits);

	if (ULevel* Level = GetRepContextLevel())
	{
		QueuedPackets[NewIndex].SeenLevelIndex = DemoDriver->ReplayHelper.FindOrAddLevelStatus(*Level).LevelIndex + 1;
	}
	else
	{
		UE_LOGF(LogDemo, Warning, "UDemoNetConnection::LowLevelSend: Missing rep context.");
	}
}

void UDemoNetConnection::TrackSendForProfiler(const void* Data, int32 NumBytes)
{
	NETWORK_PROFILER(GNetworkProfiler.FlushOutgoingBunches(this));

	// Track "socket send" even though we're not technically sending to a socket, to get more accurate information in the profiler.
	NETWORK_PROFILER(GNetworkProfiler.TrackSocketSendToCore(TEXT("Unreal"), Data, NumBytes, NumPacketIdBits, NumBunchBits, NumAckBits, NumPaddingBits, this));
}

FString UDemoNetConnection::LowLevelDescribe()
{
	return TEXT("Demo recording/playback driver connection");
}

int32 UDemoNetConnection::IsNetReady(bool Saturate)
{
	return 1;
}

void UDemoNetConnection::FlushNet(bool bIgnoreSimulation)
{
	// in playback, there is no data to send except
	// channel closing if an error occurs.
	if (GetDriver()->ServerConnection != nullptr)
	{
		InitSendBuffer();
	}
	else
	{
		Super::FlushNet(bIgnoreSimulation);
	}
}

void UDemoNetConnection::HandleClientPlayer(APlayerController* PC, UNetConnection* NetConnection)
{
	UDemoNetDriver* DemoDriver = GetDriver();

	// If the spectator is the same, assume this is for scrubbing, and we are keeping the old one
	// (so don't set the position, since we want to persist all that)
	if (DemoDriver->SpectatorController == PC)
	{
		DemoDriver->RestoreConnectionPostScrub(PC, NetConnection);
		DemoDriver->SetSpectatorController(PC);
		return;
	}

	ULocalPlayer* LocalPlayer = nullptr;
	uint8 PlayerIndex = 0;
	// Attempt to find the player that doesn't already have a connection.
	for (FLocalPlayerIterator LocalPlayerIt(GEngine, Driver->GetWorld()); LocalPlayerIt; ++LocalPlayerIt, PlayerIndex++)
	{
		if (PC->NetPlayerIndex == PlayerIndex)
		{
			LocalPlayer = *LocalPlayerIt;
			break;
		}
	}

	if (LocalPlayer != nullptr)
	{
		Super::HandleClientPlayer(PC, NetConnection);
	}
	else
	{
		DemoDriver->RestoreConnectionPostScrub(PC, NetConnection);
	}

	// This is very likely our main demo controller.
	DemoDriver->SetSpectatorController(PC);

	// Find a player start, if one exists
	if (TActorIterator<APlayerStart> PlayerStartIt(Driver->World); PlayerStartIt)
	{
		PC->SetInitialLocationAndRotation(PlayerStartIt->GetActorLocation(), PlayerStartIt->GetActorRotation());
	}
}

TSharedPtr<const FInternetAddr> UDemoNetConnection::GetRemoteAddr()
{
	return FInternetAddrDemo::DemoInternetAddr;
}

bool UDemoNetConnection::ClientHasInitializedLevel(const ULevel* TestLevel) const
{
	// We save all currently streamed levels into the demo stream so we can force the demo playback client
	// to stay in sync with the recording server
	// This may need to be tweaked or re-evaluated when we start recording demos on the client
	return (GetDriver()->GetDemoFrameNum() > 2 || Super::ClientHasInitializedLevel(TestLevel));
}

TSharedPtr<FObjectReplicator> UDemoNetConnection::CreateReplicatorForNewActorChannel(UObject* Object)
{
	TSharedPtr<FObjectReplicator> NewReplicator = MakeShareable(new FObjectReplicator());

	// To handle rewinding net startup actors in replays properly, we need to
	// initialize the shadow state with the object's current state.
	// Afterwards, we will copy the CDO state to object's current state with repnotifies disabled.
	UDemoNetDriver* NetDriver = GetDriver();
	AActor* Actor = Cast<AActor>(Object);

	const bool bIsCheckpointStartupActor = NetDriver && NetDriver->IsLoadingCheckpoint() && Actor && Actor->IsNetStartupActor();
	const bool bUseDefaultState = !bIsCheckpointStartupActor;

	NewReplicator->InitWithObject(Object, this, bUseDefaultState);

	// Now that the shadow state is initialized, copy the CDO state into the actor state.
	if (bIsCheckpointStartupActor && NewReplicator->RepLayout.IsValid() && Object->GetClass())
	{
		FRepObjectDataBuffer ObjectData(Object);
		FConstRepObjectDataBuffer ShadowData(Object->GetClass()->GetDefaultObject());

		NewReplicator->RepLayout->DiffProperties(nullptr, ObjectData, ShadowData, EDiffPropertiesFlags::Sync);

		// Need to swap roles for the startup actor since in the CDO they aren't swapped, and the CDO just
		// overwrote the actor state.
		if (Actor && (Actor->GetLocalRole() == ROLE_Authority))
		{
			Actor->SwapRoles();
		}
	}

	QueueNetStartupActorForRewind(Actor);

	return NewReplicator;
}

void UDemoNetConnection::DestroyIgnoredActor(AActor* Actor)
{
	QueueNetStartupActorForRewind(Actor);

	Super::DestroyIgnoredActor(Actor);
}

void UDemoNetConnection::QueueNetStartupActorForRewind(AActor* Actor)
{
	UDemoNetDriver* NetDriver = GetDriver();

	// Handle rewinding initially dormant startup actors that were changed on the client
	const bool bIsStartupActor = NetDriver && Actor && Actor->IsNetStartupActor() && !Actor->bReplayRewindable;
	if (bIsStartupActor)
	{
		NetDriver->QueueNetStartupActorForRollbackViaDeletion(Actor);
	}
}

void UDemoNetConnection::NotifyActorNetGUID(UActorChannel* Channel)
{
	const UDemoNetDriver* const NetDriver = GetDriver();

	if (Channel && NetDriver && NetDriver->HasDeltaCheckpoints())
	{
		GetOpenChannelMap().Add(Channel->ActorNetGUID, Channel);
	}
}

void UDemoNetConnection::NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason)
{
	const UDemoNetDriver* const NetDriver = GetDriver();

	if (Channel && NetDriver && NetDriver->HasDeltaCheckpoints())
	{
		GetOpenChannelMap().Remove(Channel->ActorNetGUID);
	}
}

bool UDemoNetDriver::IsLevelInitializedForActor(const AActor* InActor, const UNetConnection* InConnection) const
{
	return (GetDemoFrameNum() > 2 || Super::IsLevelInitializedForActor(InActor, InConnection));
}

bool UDemoNetDriver::IsPlayingClientReplay() const
{
	return IsPlaying() && EnumHasAnyFlags(ReplayHelper.PlaybackDemoHeader.HeaderFlags, EReplayHeaderFlags::ClientRecorded);
}

void UDemoNetDriver::NotifyGotoTimeFinished(bool bWasSuccessful)
{
	// execute and clear the transient delegate
	OnGotoTimeDelegate_Transient.ExecuteIfBound(bWasSuccessful);
	OnGotoTimeDelegate_Transient.Unbind();

	// execute and keep the permanent delegate
	// call only when successful
	if (bWasSuccessful)
	{
		FNetworkReplayDelegates::OnReplayScrubComplete.Broadcast(World);
	}
}

void UDemoNetDriver::OnSeamlessTravelStartDuringRecording(const FString& LevelName)
{
	if (ClientConnections.Num() > 0)
	{
		ReplayHelper.OnSeamlessTravelStart(World, LevelName, ClientConnections[0]);
	}
}

void UDemoNetDriver::InitDestroyedStartupActors()
{
	Super::InitDestroyedStartupActors();

	if (World)
	{
		check(ReplayHelper.RecordingDeletedNetStartupActors.Num() == 0);
		check(ReplayHelper.RecordingDeltaCheckpointData.RecordingDeletedNetStartupActors.Num() == 0);

		// add startup actors destroyed before the creation of this net driver
		for (FConstLevelIterator LevelIt(World->GetLevelIterator()); LevelIt; ++LevelIt)
		{
			if (const ULevel* Level = *LevelIt)
			{
				const TArray<FReplicatedStaticActorDestructionInfo>& DestroyedReplicatedStaticActors = Level->GetDestroyedReplicatedStaticActors();
				for (const FReplicatedStaticActorDestructionInfo& Info : DestroyedReplicatedStaticActors)
				{
					ReplayHelper.RecordingDeletedNetStartupActors.Add(Info.FullName);
					ReplayHelper.RecordingDeltaCheckpointData.RecordingDeletedNetStartupActors.Add(Info.FullName);
				}
			}
		}
	}
}

void UDemoNetDriver::OnActorPreDestroy(AActor* DestroyedActor)
{
	check(DestroyedActor);

	if (IsRecording())
	{
		ReplayHelper.RemoveActorFromCheckpoint(ClientConnections[0], DestroyedActor);
	}
}

void UDemoNetDriver::NotifyActorDestroyed(AActor* Actor, bool IsSeamlessTravel, EChannelCloseReason CloseReason)
{
	check(Actor != nullptr);

	const bool bIsRecording = IsRecording();
	const bool bNetStartup = Actor->IsNetStartupActor();
	const bool bActorRewindable = Actor->bReplayRewindable;

	if (bActorRewindable && !IsSeamlessTravel && !bIsRecording)
	{
		if (bNetStartup || !TrackedRewindActorsByGUID.Contains(GetNetGuidCache()->NetGUIDLookup.FindRef(Actor)))
		{
			// This may happen during playback due to new versions of code playing captures with old versions.
			// but this should never happen during recording (otherwise it's likely a game code bug). 
			// We catch that case below.
			UE_LOGF(LogDemo, Warning, "Replay Rewindable Actor destroyed during playback. Replay may show artifacts (%ls)", *Actor->GetFullName());
		}
	}

	if (bIsRecording)
	{
		// We don't want to send any destruction info in this case, because the actor should stick around.
		// The Replay will manage deleting this when it performs streaming or travel behavior.
		if (bNetStartup && IsSeamlessTravel)
		{
			// This is a stripped down version of UNetDriver::NotifyActorDestroy and UActorChannel::Close
			// combined, and should be kept up to date with those methods.
			if (UNetConnection* Connection = ClientConnections[0])
			{
				if (Actor->bNetTemporary)
				{
					Connection->SentTemporaries.Remove(Actor);
				}

				if (UActorChannel* Channel = Connection->FindActorChannelRef(Actor))
				{
					check(Channel->OpenedLocally);
					Channel->bClearRecentActorRefs = false;
					Channel->SetClosingFlag();
					Channel->ReleaseReferences(false);
				}

				Connection->NotifyActorDestroyed(Actor, IsSeamlessTravel);
			}

			GetNetworkObjectList().Remove(Actor);
			RenamedStartupActors.Remove(Actor->GetFName());
			return;
		}

		if (!IsSeamlessTravel)
		{
			ReplayHelper.NotifyActorDestroyed(ClientConnections[0], Actor);
		}
	}

	Super::NotifyActorDestroyed(Actor, IsSeamlessTravel, CloseReason);
}

void UDemoNetDriver::NotifyActorTornOff(AActor* Actor)
{
	if (IsRecording() && IsValid(Actor))
	{
		ForcePropertyCompare(Actor);

		// Replicate one last time to the replay stream
		ReplayHelper.ReplicateActor(Actor, ClientConnections[0], true);

		// Handle being pending in a checkpoint save
		ReplayHelper.RemoveActorFromCheckpoint(ClientConnections[0], Actor);
	}

	Super::NotifyActorTornOff(Actor);
}

void UDemoNetDriver::CleanupOutstandingRewindActors()
{
	if (World)
	{
		for (const FNetworkGUID& NetGUID : TrackedRewindActorsByGUID)
		{
			if (FNetGuidCacheObject* CacheObject = GetNetGuidCache()->ObjectLookup.Find(NetGUID))
			{
				if (AActor* Actor = Cast<AActor>(CacheObject->Object))
				{
					// Destroy the actor before removing entries from the GuidCache so its entries are still valid in NotifyActorDestroyed.
					World->DestroyActor(Actor, true);

					ensureMsgf(GetNetGuidCache()->NetGUIDLookup.Remove(CacheObject->Object) > 0, TEXT("CleanupOutstandingRewindActors: No entry found for %s in NetGUIDLookup"), *NetGUID.ToString());
					GetNetGuidCache()->ObjectLookup.Remove(NetGUID);
					CacheObject->bNoLoad = false;
				}
				else
				{
					UE_LOGF(LogDemo, Warning, "CleanupOutstandingRewindActors - Invalid object for %ls, skipping.", *NetGUID.ToString());
					continue;
				}
			}	
			else
			{
				UE_LOGF(LogDemo, Warning, "CleanupOutstandingRewindActors - CacheObject not found for %ls", *NetGUID.ToString());
			}
		}
	}

	TrackedRewindActorsByGUID.Empty();
}

void UDemoNetDriver::NotifyActorChannelOpen(UActorChannel* Channel, AActor* Actor)
{
	const bool bValidChannel = ensureMsgf(Channel, TEXT("NotifyActorChannelOpen called with invalid channel"));
	const bool bValidActor = ensureMsgf(Actor, TEXT("NotifyActorChannelOpen called with invalid actor"));
	
	// Rewind the actor if necessary.
	// This should be called before any other notifications / data reach the Actor.
	if (bValidChannel && bValidActor && TrackedRewindActorsByGUID.Remove(Channel->ActorNetGUID) > 0)
	{
		Actor->RewindForReplay();
	}	

	// Only necessary on clients where dynamic actors can go in and out of relevancy
	if (bValidChannel && bValidActor && IsRecording() && HasDeltaCheckpoints())
	{
		ReplayHelper.RecordingDeltaCheckpointData.DestroyedDynamicActors.Remove(Channel->ActorNetGUID);
	}
}

void UDemoNetDriver::NotifyActorClientDormancyChanged(AActor* Actor, ENetDormancy OldDormancyState)
{
	if (IsRecording() && (Actor->NetDormancy <= DORM_Awake))
	{
		AddNetworkActor(Actor);
		FlushActorDormancy(Actor);

		GetNetworkObjectList().MarkActive(Actor, ClientConnections[0], this);
		GetNetworkObjectList().ClearRecentlyDormantConnection(Actor, ClientConnections[0], this);
	}
}

void UDemoNetDriver::NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason)
{
	// channels can be cleaned up during the checkpoint record (dormancy), make sure to skip those
	if (IsRecording() && HasDeltaCheckpoints() && (ReplayHelper.GetCheckpointSaveState() == FReplayHelper::ECheckpointSaveState::Idle))
	{
		if (Channel && Channel->bOpenedForCheckpoint)
		{
			ReplayHelper.RecordingDeltaCheckpointData.ChannelsToClose.Add(Channel->ActorNetGUID, CloseReason);
		}
	}

	Super::NotifyActorChannelCleanedUp(Channel, CloseReason);
}

void UDemoNetDriver::NotifyActorLevelUnloaded(AActor* Actor)
{
	if (ServerConnection != nullptr)
	{
		// This is a combination of the Client and Server logic for destroying a channel,
		// since we won't actually be sending data back and forth.
		if (UActorChannel* ActorChannel = ServerConnection->FindActorChannelRef(Actor))
		{
			ServerConnection->RemoveActorChannel(Actor);
			ActorChannel->ReleaseReferences(false);
			ActorChannel->ConditionalCleanUp(false, EChannelCloseReason::LevelUnloaded);
		}
	}

	Super::NotifyActorLevelUnloaded(Actor);
}

void UDemoNetDriver::QueueNetStartupActorForRollbackViaDeletion(AActor* Actor)
{
	if (!IsPlaying())
	{
		return;		// We should only be doing this at runtime while playing a replay
	}

	if (bSkipStartupActorRollback)
	{
		return;
	}

	check(Actor != nullptr);

	if (!Actor->IsNetStartupActor())
	{
		return;		// We only want startup actors
	}

	if (Actor->bReplayRewindable)
	{
		UE_LOGF(LogDemo, Warning, "Attempted to queue a Replay Rewindable Actor for Rollback Via Deletion. Replay may have artifacts (%ls)", *GetFullNameSafe(Actor));
		return;
	}

	FString ActorFullName = Actor->GetFullName();
	if (RollbackNetStartupActors.Contains(ActorFullName))
	{
		return;		// This actor is already queued up
	}

	FRollbackNetStartupActorInfo& RollbackActor = RollbackNetStartupActors.Add(MoveTemp(ActorFullName));

	RollbackActor.Name		= Actor->GetFName();
	RollbackActor.Archetype	= Actor->GetArchetype();
	RollbackActor.Location	= Actor->GetActorLocation();
	RollbackActor.Rotation	= Actor->GetActorRotation();
	RollbackActor.Scale3D	= Actor->GetActorScale3D();

	if (ULevel* ActorLevel = Actor->GetLevel())
	{
		RollbackActor.LevelName = ActorLevel->GetOutermost()->GetFName();
	}

	if (GDemoSaveRollbackActorState != 0)
	{
		// Save actor state
		{
			TSharedPtr<FObjectReplicator> NewReplicator = MakeShared<FObjectReplicator>();
			NewReplicator->InitWithObject(Actor->GetArchetype(), ServerConnection, false);

			if (NewReplicator->RepLayout.IsValid() && NewReplicator->RepState.IsValid())
			{
				FReceivingRepState* ReceivingRepState = NewReplicator->RepState->GetReceivingRepState();
				FRepShadowDataBuffer ShadowData(ReceivingRepState->StaticBuffer.GetData());
				FConstRepObjectDataBuffer ActorData(Actor);

				if (NewReplicator->RepLayout->DiffStableProperties(nullptr, ToRawPtr(MutableView(RollbackActor.ObjReferences)), ShadowData, ActorData))
				{
					RollbackActor.RepState = MakeShareable(NewReplicator->RepState.Release());
				}
			}
		}

		// Save component state
		for (UActorComponent* ActorComp : Actor->GetComponents())
		{
			if (ActorComp)
			{
				TSharedPtr<FObjectReplicator> SubObjReplicator = MakeShared<FObjectReplicator>();
				SubObjReplicator->InitWithObject(ActorComp->GetArchetype(), ServerConnection, false);

				if (SubObjReplicator->RepLayout.IsValid() && SubObjReplicator->RepState.IsValid())
				{
					FReceivingRepState* ReceivingRepState = SubObjReplicator->RepState->GetReceivingRepState();
					FRepShadowDataBuffer ShadowData(ReceivingRepState->StaticBuffer.GetData());
					FConstRepObjectDataBuffer ActorCompData(ActorComp);

					if (SubObjReplicator->RepLayout->DiffStableProperties(nullptr, ToRawPtr(MutableView(RollbackActor.ObjReferences)), ShadowData, ActorCompData))
					{
						RollbackActor.SubObjRepState.Add(ActorComp->GetFullName(), MakeShareable(SubObjReplicator->RepState.Release()));
					}
				}
			}
		}
	}
}

void UDemoNetDriver::ForceNetUpdate(AActor* Actor)
{
	UReplicationDriver* RepDriver = GetReplicationDriver();
	if (RepDriver)
	{
		RepDriver->ForceNetUpdate(Actor);
	}
	else
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FNetworkObjectInfo* NetActor = FindNetworkObjectInfo(Actor))
		{
			// replays use update times relative to DemoCurrentTime and not World->TimeSeconds
			NetActor->NextUpdateTime = GetDemoCurrentTime() - 0.01f;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

UChannel* UDemoNetDriver::InternalCreateChannelByName(const FName& ChName)
{
	// In case of recording off the game thread with CVarDemoClientRecordAsyncEndOfFrame,
	// we need to clear the async flag on the channel so that it will get cleaned up by GC.
	// This should be safe since channel objects don't interact with async loading, and
	// async recording happens in a very controlled manner.
	UChannel* NewChannel = Super::InternalCreateChannelByName(ChName);
	if (NewChannel)
	{
		NewChannel->ClearInternalFlags(EInternalObjectFlags::Async);
	}
	return NewChannel;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDemoNetDriver::NotifyDemoPlaybackFailure(EDemoPlayFailure::Type FailureType)
{
	UE_LOGF(LogDemo, Warning, "NotifyDemoPlaybackFailure: %ls", EDemoPlayFailure::ToString(FailureType));

	const bool bIsPlaying = IsPlaying();

	// fire delegate
	FNetworkReplayDelegates::OnReplayStartFailure.Broadcast(World, FailureType);
	FNetworkReplayDelegates::OnReplayPlaybackFailure.Broadcast(World, EReplayResult::Unknown);

	StopDemo();

	if (bIsPlaying && World)
	{
		if (UGameInstance* GameInstance = World->GetGameInstance())
		{
			GameInstance->HandleDemoPlaybackFailure(EReplayResult::Unknown);
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UDemoNetDriver::NotifyDemoPlaybackError(const UE::Net::TNetResult<EReplayResult>& Result)
{
	const bool bIsPlaying = IsPlaying();

	// fire delegate
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNetworkReplayDelegates::OnReplayStartFailure.Broadcast(World, EDemoPlayFailure::Generic);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FNetworkReplayDelegates::OnReplayPlaybackFailure.Broadcast(World, Result);

	StopDemo();

	if (bIsPlaying && World)
	{
		if (UGameInstance* GameInstance = World->GetGameInstance())
		{
			GameInstance->HandleDemoPlaybackFailure(Result);
		}
	}
}

void UDemoNetDriver::NotifyDemoRecordFailure(const UE::Net::TNetResult<EReplayResult>& Result)
{
	const bool bIsRecording = IsRecording();

	StopDemo();

	if (bIsRecording && !PendingRecordFailure.IsSet())
	{
		PendingRecordFailure.Emplace(Result);
	}
}

FString UDemoNetDriver::GetDemoPath() const
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		FString DemoPath;
		if (ReplayHelper.ReplayStreamer->GetDemoPath(DemoPath) == EStreamingOperationResult::Success)
		{
			return DemoPath;
		}
	}

	return FString();
}

bool UDemoNetDriver::ShouldReplicateFunction(AActor* Actor, UFunction* Function) const
{
	// ReplayNetConnection does not currently have this functionality, as it filters fast shared rpcs directly in the rep graph
	bool bShouldRecordMulticast = (Function && Function->FunctionFlags & FUNC_NetMulticast) && IsRecording();
	if (bShouldRecordMulticast)
	{
		const FString FuncPathName = GetPathNameSafe(Function);
		const int32 Idx = MulticastRecordOptions.IndexOfByPredicate([FuncPathName](const FMulticastRecordOptions& Options) { return (Options.FuncPathName == FuncPathName); });
		if (Idx != INDEX_NONE)
		{
			if (World && World->IsRecordingClientReplay())
			{
				bShouldRecordMulticast = bShouldRecordMulticast && !MulticastRecordOptions[Idx].bClientSkip;
			}
			else
			{
				bShouldRecordMulticast = bShouldRecordMulticast && !MulticastRecordOptions[Idx].bServerSkip;
			}
		}
	}

	return bShouldRecordMulticast || Super::ShouldReplicateFunction(Actor, Function);
}

bool UDemoNetDriver::ShouldReplicateActor(AActor* Actor) const
{
	// replicate actors that share the demo net driver name, or actors belonging to the game net driver
	return (Actor && (Actor->GetIsReplicated() || Actor->GetTearOff())) && (Super::ShouldReplicateActor(Actor) || (Actor->GetNetDriverName() == NAME_GameNetDriver));
}

/*
* If a large number of Actors makes it onto the NetworkObjectList, and Demo Recording is limited,
* then we can easily hit cases where building the Consider List and Sorting it can take up the
* entire time slice. In that case, we'll have spent a lot of time setting up for replication,
* but never actually doing it.
* Further, if dormancy is used, dormant actors need to replicate once before they're removed from
* the NetworkObjectList. That means in the worst case, we can have a large number of dormant actors
* artificially driving up consider / sort times.
*
* To prevent that, we'll throttle the amount of time we spend prioritize next frame based
* on how much time it took this frame.
*
* @param TimeSlicePercent	The current percent of time allocated to building consider lists / prioritizing.
* @param ReplicatedPercet	The percent of actors that were replicated this last frame.
*/
void UDemoNetDriver::AdjustConsiderTime(const float ReplicatedPercent)
{
	if (MaxDesiredRecordTimeMS > 0.f)
	{
		auto ConditionallySwap = [](float& Less, float& More)
		{
			if (More < Less)
			{
				Swap(Less, More);
			}
		};

		float DecreaseThreshold = CVarDemoDecreaseRepPrioritizeThreshold.GetValueOnAnyThread();
		float IncreaseThreshold = CVarDemoIncreaseRepPrioritizeThreshold.GetValueOnAnyThread();
		ConditionallySwap(DecreaseThreshold, IncreaseThreshold);

		float MinRepTime = CVarDemoMinimumRepPrioritizeTime.GetValueOnAnyThread();
		float MaxRepTime = CVarDemoMaximumRepPrioritizeTime.GetValueOnAnyThread();
		ConditionallySwap(MinRepTime, MaxRepTime);
		MinRepTime = FMath::Clamp<float>(MinRepTime, 0.1, 1.0);
		MaxRepTime = FMath::Clamp<float>(MaxRepTime, 0.1, 1.0);

		if (ReplicatedPercent > IncreaseThreshold)
		{
			RecordBuildConsiderAndPrioritizeTimeSlice += 0.1f;
			UE_LOGF(LogDemo, Verbose, "AdjustConsiderTime: RecordBuildConsiderAndPrioritizeTimeSlice is now %0.1f", RecordBuildConsiderAndPrioritizeTimeSlice)
		}
		else if (ReplicatedPercent < DecreaseThreshold)
		{
			RecordBuildConsiderAndPrioritizeTimeSlice *= (1.f - ReplicatedPercent) * 0.5f;
			UE_LOGF(LogDemo, Verbose, "AdjustConsiderTime: RecordBuildConsiderAndPrioritizeTimeSlice is now %0.1f", RecordBuildConsiderAndPrioritizeTimeSlice)
		}

		RecordBuildConsiderAndPrioritizeTimeSlice = FMath::Clamp<float>(RecordBuildConsiderAndPrioritizeTimeSlice, MinRepTime, MaxRepTime);
	}
}

/*-----------------------------------------------------------------------------
	UDemoPendingNetGame.
-----------------------------------------------------------------------------*/

UDemoPendingNetGame::UDemoPendingNetGame(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UDemoPendingNetGame::Tick(float DeltaTime)
{
	// Replays don't need to do anything here
}

void UDemoPendingNetGame::SendJoin()
{
	// Don't send a join request to a replay
}

bool UDemoPendingNetGame::LoadMapCompleted(UEngine* Engine, FWorldContext& Context, bool bInLoadedMapSuccessfully, const FString& LoadMapError)
{
	bLoadedMapSuccessfully = bInLoadedMapSuccessfully;
	UDemoNetDriver* TheDriver = GetDemoNetDriver();

	// If we have a demo pending net game we should have a demo net driver
	check(TheDriver);

	if (!bLoadedMapSuccessfully)
	{
		TheDriver->StopDemo();

		// If we don't have a world that means we failed loading the new world.
		// Since there is no world, we must free the net driver ourselves
		// Technically the pending net game should handle it, but things aren't quite setup properly to handle that either
		if (Context.World() == nullptr)
		{
			GEngine->DestroyNamedNetDriver(Context.PendingNetGame, TheDriver->NetDriverName);
		}

		Context.PendingNetGame = nullptr;

		GEngine->BrowseToDefaultMap(Context);

		UE_LOGF(LogDemo, Error, "UDemoPendingNetGame::HandlePostLoadMap: LoadMap failed: %ls", *LoadMapError);

		if (Context.OwningGameInstance)
		{
			Context.OwningGameInstance->HandleDemoPlaybackFailure(EReplayResult::LoadMap);
		}
		// we already handled the error so don't need the TickPendingNetGame to handle it 
		return true;
	}
	return true;
}

void UDemoNetDriver::Serialize(FArchive& Ar)
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UDemoNetDriver::Serialize");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Super", Super::Serialize(Ar));

	if (Ar.IsCountingMemory())
	{
		// TODO: We don't currently track:
		//		Replay Streamers
		//		Dynamic Delegate Data
		//		QueuedReplayTasks.
		//		DemoURL

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PlaybackPackets",
			PlaybackPackets.CountBytes(Ar);
			for (const FPlaybackPacket& Packet : PlaybackPackets)
			{
				Packet.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NonQueuedGUIDsForScrubbing", NonQueuedGUIDsForScrubbing.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueuedReplayTasks", QueuedReplayTasks.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DemoSessionID", DemoSessionID.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PrioritizedActors", PrioritizedActors.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LevelInternals", LevelIntervals.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("TrackedRewindActorsByGUID", TrackedRewindActorsByGUID.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueuedPacketsBeforeTravel",
			QueuedPacketsBeforeTravel.CountBytes(Ar);
			for (const FQueuedDemoPacket& QueuedPacket : QueuedPacketsBeforeTravel)
			{
				QueuedPacket.CountBytes(Ar);
			}
		);

		ReplayHelper.Serialize(Ar);
	}
}

void UDemoNetConnection::Serialize(FArchive& Ar)
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UDemoNetConnection::Serialize");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Super", Super::Serialize(Ar));
}

void UDemoNetDriver::SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider)
{
	Super::SetAnalyticsProvider(InProvider);

	ReplayHelper.SetAnalyticsProvider(InProvider);
}

void UDemoNetDriver::SetWorld(UWorld* InWorld)
{
	if (World)
	{
		World->RemoveOnActorDestroyedHandler(DelegateHandleActorPreDestroy);
	}

	Super::SetWorld(InWorld);

	ReplayHelper.World = InWorld;

	if (InWorld)
	{
		DelegateHandleActorPreDestroy = InWorld->AddOnActorDestroyedHandler(FOnActorDestroyed::FDelegate::CreateUObject(this, &UDemoNetDriver::OnActorPreDestroy));
	}
}

bool UDemoNetDriver::ShouldForwardFunction(AActor* Actor, UFunction* Function, void* Parms) const
{
	// currently no need to forward replay playback RPCs on to other drivers
	return false;
}

void UDemoNetDriver::RequestCheckpoint()
{
	if (IsRecording())
	{
		ReplayHelper.RequestCheckpoint();
	}
}

