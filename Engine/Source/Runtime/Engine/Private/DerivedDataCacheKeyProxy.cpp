// Copyright Epic Games, Inc. All Rights Reserved.

#include "DerivedDataCacheKeyProxy.h"

#if WITH_EDITORONLY_DATA

#include "DerivedDataCacheKey.h"

namespace UE::DerivedData
{

static_assert(sizeof(FCacheKeyProxy) == sizeof(FCacheKeyProxy));
static_assert(alignof(FCacheKeyProxy) == alignof(FCacheKeyProxy));

FCacheKeyProxy::FCacheKeyProxy(const FCacheKey& InKey)
{
	new(AsCacheKey()) FCacheKey(InKey);
}

FCacheKeyProxy::~FCacheKeyProxy()
{
	AsCacheKey()->~FCacheKey();
}

} // UE::DerivedData

#endif // WITH_EDITORONLY_DATA