// Copyright Epic Games, Inc. All Rights Reserved.

#include "DestructibleInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DestructibleInterface)

UDestructibleInterface::UDestructibleInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

