// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceProfiles/DeviceProfile.h"
#include "Misc/Paths.h"
#include "Scalability.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectIterator.h"
#include "HAL/IConsoleManager.h"

#include "DeviceProfiles/DeviceProfileFragment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeviceProfile)

DEFINE_LOG_CATEGORY_STATIC(LogDeviceProfile, Log, All);

UDeviceProfileFragment::UDeviceProfileFragment(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


UDeviceProfile::UDeviceProfile(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BaseProfileName = TEXT("");
	DeviceType = TEXT("");
	bIsVisibleForAssets = false;

	bVisible = true;

	FString DeviceProfileFileName = FPaths::EngineConfigDir() + TEXT("Deviceprofiles.ini");
//	LoadConfig(GetClass(), *DeviceProfileFileName, UE::LCPF_ReadParentSections);
}

const FSelectedFragmentProperties* UDeviceProfile::GetFragmentByTag(FName& FragmentTag) const
{
	for (const FSelectedFragmentProperties& SelectedFragment : SelectedFragments)
	{
		if (SelectedFragment.Tag == FragmentTag)
		{
			return &SelectedFragment;
		}
	}
	return nullptr;
}

void UDeviceProfile::GatherParentCVarInformationRecursively(OUT TMap<FString, FString>& CVarInformation) const
{
	// Recursively build the parent tree
	if (!BaseProfileName.IsEmpty())
	{
		UDeviceProfile* ParentProfile = GetParentProfile(false);
		check(ParentProfile != nullptr);

		for (auto& CurrentCVar : ParentProfile->CVars)
		{
			FString CVarKey, CVarValue;
			if (CurrentCVar.Split(TEXT("="), &CVarKey, &CVarValue))
			{
				if (CVarInformation.Find(CVarKey) == nullptr)
				{
					CVarInformation.Add(CVarKey, *CurrentCVar);
				}
			}
		}

		ParentProfile->GatherParentCVarInformationRecursively(CVarInformation);
	}
}


#if ALLOW_OTHER_PLATFORM_CONFIG
ENGINE_API void UDeviceProfile::SetPreviewTextureGroupRedirect(const UTextureLODSettings* InPreviewTextureGroupRedirect)
{
	PreviewTextureGroupRedirect = InPreviewTextureGroupRedirect;
}
#endif

UTextureLODSettings* UDeviceProfile::GetTextureLODSettings() const
{
	const UTextureLODSettings * Ret = this; // inheritance
	
#if ALLOW_OTHER_PLATFORM_CONFIG
	if ( PreviewTextureGroupRedirect )
	{
		Ret = PreviewTextureGroupRedirect;
	}
#endif

	return const_cast<UTextureLODSettings *>(Ret);
}

UDeviceProfile* UDeviceProfile::GetParentProfile(bool bIncludeDefaultObject) const
{
	UDeviceProfile* ParentProfile = nullptr;

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		if (Parent != nullptr)
		{
			return Parent;
		}
		if (!BaseProfileName.IsEmpty())
		{
			ParentProfile = FindObject<UDeviceProfile>(GetTransientPackage(), *BaseProfileName);
		}
		// don't find a parent for GlobalDefaults as it's the implied parent for everything (it would
		// return itself which is bad)
		if (!ParentProfile && bIncludeDefaultObject && GetName() != TEXT("GlobalDefaults"))
		{
			ParentProfile = FindObject<UDeviceProfile>(GetTransientPackage(), TEXT("GlobalDefaults"));
		}
	}

	return ParentProfile;
}

void UDeviceProfile::BeginDestroy()
{
	UE_LOGF(LogDeviceProfile, Verbose, "Device profile begin destroy: [%p] %ls", this, *GetName());

	Super::BeginDestroy();
}

void UDeviceProfile::ValidateProfile()
{
	ValidateTextureLODGroups();
}

void UDeviceProfile::ValidateTextureLODGroups()
{
	// Ensure the Texture LOD Groups are in order of TextureGroup Enum
	TextureLODGroups.Sort([]
		(const FTextureLODGroup& Lhs, const FTextureLODGroup& Rhs)
		{
			return (int32)Lhs.Group < (int32)Rhs.Group;
		}
	);

	// Make sure every Texture Group has an entry, any that aren't specified for this profile should use it's parents values, or the defaults.
	UDeviceProfile* ParentProfile = GetParentProfile(true);

	for (int32 GroupId = 0; GroupId < (int32)TEXTUREGROUP_MAX; ++GroupId)
	{
		if (TextureLODGroups.Num() < (GroupId + 1) || TextureLODGroups[GroupId].Group > GroupId)
		{
			if (ParentProfile && (ParentProfile->TextureLODGroups.Num() > GroupId))
			{
				TextureLODGroups.Insert(ParentProfile->TextureLODGroups[GroupId], GroupId);
			}
			else
			{
				TextureLODGroups.Insert(FTextureLODGroup(), GroupId);
			}

			TextureLODGroups[GroupId].Group = (TextureGroup)GroupId;
		}
	}

#define SETUPLODGROUP(GroupId) SetupLODGroup(GroupId);
	FOREACH_ENUM_TEXTUREGROUP(SETUPLODGROUP)
#undef SETUPLODGROUP

	// Propagate value from parent if the property isn't set.
	if (ParentProfile && (TextureGroupOptionalQualityLevel == -1))
	{
		TextureGroupOptionalQualityLevel = ParentProfile->TextureGroupOptionalQualityLevel;
	}
}

#if WITH_EDITOR

void UDeviceProfile::HandleCVarsChanged()
{
	OnCVarsUpdated().ExecuteIfBound();
	FScopeLock Lock(&CVarsLock);
	ConsolidatedCVars.Reset();
}

void UDeviceProfile::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty( PropertyChangedEvent );

	if( PropertyChangedEvent.Property->GetFName() == TEXT("BaseProfileName") )
	{
		FString NewParentName = *PropertyChangedEvent.Property->ContainerPtrToValuePtr<FString>( this );

		if( UObject* ParentRef = FindObject<UDeviceProfile>( GetTransientPackage(), *NewParentName ) )
		{
			// Generation and profile reference
			TMap<UDeviceProfile*,int32> DependentProfiles;

			int32 NumGenerations = 1;
			DependentProfiles.Add(this,0);

			for( TObjectIterator<UDeviceProfile> DeviceProfileIt; DeviceProfileIt; ++DeviceProfileIt )
			{
				UDeviceProfile* ParentProfile = *DeviceProfileIt;

				if( IsValid(ParentProfile) )
				{
					int32 ProfileGeneration = 1;
					do
					{
						if( this->GetName() == ParentProfile->BaseProfileName )
						{
							NumGenerations = NumGenerations > ProfileGeneration ? NumGenerations : ProfileGeneration;
							DependentProfiles.Add(*DeviceProfileIt,ProfileGeneration);
							break;
						}

						ParentProfile = ParentProfile->GetParentProfile(false);
						++ProfileGeneration;
					} while ( ParentProfile );
				}
			}


			UDeviceProfile* ClassCDO = CastChecked<UDeviceProfile>(GetClass()->GetDefaultObject());

			for( int32 CurrentGeneration = 0; CurrentGeneration < NumGenerations; CurrentGeneration++ )
			{
				for( TMap<UDeviceProfile*,int32>::TIterator DeviceProfileIt(DependentProfiles); DeviceProfileIt; ++DeviceProfileIt )
				{
					if( CurrentGeneration == DeviceProfileIt.Value() )
					{
						UDeviceProfile* CurrentGenerationProfile = DeviceProfileIt.Key();
						UDeviceProfile* ParentProfile = CurrentGenerationProfile->GetParentProfile(true);

						for (TFieldIterator<FProperty> CurrentObjPropertyIter( GetClass() ); CurrentObjPropertyIter; ++CurrentObjPropertyIter)
						{
							bool bIsSameParent = CurrentObjPropertyIter->Identical_InContainer( ClassCDO, CurrentGenerationProfile );
							if( bIsSameParent )
							{
								void* CurrentGenerationProfilePropertyAddress = CurrentObjPropertyIter->ContainerPtrToValuePtr<void>( CurrentGenerationProfile );
								void* ParentPropertyAddr = CurrentObjPropertyIter->ContainerPtrToValuePtr<void>( ParentRef );

								CurrentObjPropertyIter->CopyCompleteValue( CurrentGenerationProfilePropertyAddress, ParentPropertyAddr );
							}
						}
					}
				}
			}
		}
		HandleCVarsChanged();
	}
	else if(PropertyChangedEvent.Property->GetFName() == TEXT("CVars"))
	{
		HandleCVarsChanged();
	}
}

bool UDeviceProfile::ModifyCVarValue(const FString& ChangeCVarName, const FString& NewCVarValue, bool bAddIfNonExistant)
{
	auto Index = CVars.IndexOfByPredicate(
		[&ChangeCVarName](const FString& CVar) {
		FString CVarName;
		CVar.Split(TEXT("="), &CVarName, NULL);
		return CVarName == ChangeCVarName;
	} );

	if (Index != INDEX_NONE)
	{
		FString CVarName;
		CVars[Index].Split(TEXT("="), &CVarName, NULL);
		check(CVarName == ChangeCVarName);
		CVars[Index] = FString::Printf(TEXT("%s=%s"), *CVarName, *NewCVarValue);

		HandleCVarsChanged();
		return true;
	}
	else if(bAddIfNonExistant)
	{
		CVars.Add(FString::Printf(TEXT("%s=%s"), *ChangeCVarName, *NewCVarValue));
		
		HandleCVarsChanged();
		return true;
	}

	return false;
}

FString UDeviceProfile::GetCVarValue(const FString& CVarName) const
{
	auto Index = CVars.IndexOfByPredicate(
		[&CVarName](const FString& CVar) {
		FString Name;
		CVar.Split(TEXT("="), &Name, NULL);
		return Name == CVarName;
	});

	if (Index != INDEX_NONE)
	{
		FString Value;
		CVars[Index].Split(TEXT("="), NULL, &Value);
		return Value;
	}
	else
	{
		return FString();
	}
}

bool UDeviceProfile::GetConsolidatedCVarValue(const TCHAR* CVarName, FString& OutString, bool bCheckDefaults /*=false*/) const
{
	FScopeLock Lock(&CVarsLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FString* FoundValue = GetConsolidatedCVars().Find(CVarName);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if(FoundValue)
	{
		OutString = *FoundValue;
		return true;
	}
	
	if(bCheckDefaults)
	{
		if(IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName))
		{
			OutString = CVar->GetString();
			return true;
		}
	}

	OutString.Empty();
	return false;
}

bool UDeviceProfile::GetConsolidatedCVarValue(const TCHAR* CVarName, int32& OutValue, bool bCheckDefaults /*=false*/) const
{
	FString StringValue;
	if(GetConsolidatedCVarValue(CVarName, StringValue))
	{
		OutValue = FCString::Atoi(*StringValue);
		return true;
	}

	if(bCheckDefaults)
	{
		if(IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName))
		{
			OutValue = CVar->GetInt();
			return true;
		}
	}

	OutValue = 0;
	return false;
}

bool UDeviceProfile::GetConsolidatedCVarValue(const TCHAR* CVarName, float& OutValue, bool bCheckDefaults /*=false*/) const
{
	FString StringValue;
	if(GetConsolidatedCVarValue(CVarName, StringValue))
	{
		OutValue = FCString::Atof(*StringValue);
		return true;
	}

	if(bCheckDefaults)
	{
		if(IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName))
		{
			OutValue = CVar->GetFloat();
			return true;
		}
	}

	OutValue = 0.0f;
	return false;
}

const TMap<FString, FString>& UDeviceProfile::GetConsolidatedCVars() const
{
	// Helper function to add a profile's CVars to the consolidated map
	auto BuildCVarMap = [](const UDeviceProfile* InProfile, TMap<FString, FString>& InOutMap)
	{
		for (const auto& CurrentCVar : InProfile->CVars)
		{
			FString CVarKey, CVarValue;
			if (CurrentCVar.Split(TEXT("="), &CVarKey, &CVarValue))
			{	
				//Prevent parents from overwriting the values in child profiles.
				if (InOutMap.Find(CVarKey) == nullptr)
				{
					InOutMap.Add(CVarKey, CVarValue);
				}
			}
		}
	};

	// First build our own CVar map
	FScopeLock Lock(&CVarsLock);
	if(ConsolidatedCVars.Num() == 0)
	{
		BuildCVarMap(this, ConsolidatedCVars);

		// Iteratively build the parent tree
		const UDeviceProfile* ParentProfile = GetParentProfile(false);
		while (ParentProfile)
		{
			BuildCVarMap(ParentProfile, ConsolidatedCVars);
			ParentProfile = ParentProfile->GetParentProfile(false);
		}
	}

	return ConsolidatedCVars;
}

void UDeviceProfile::ForEachConsolidatedCVar(const TFunctionRef<bool(const FString&, const FString&)>& FuncRef) const
{
	FScopeLock Lock(&CVarsLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TMap<FString, FString>& ConsolidatedCvars = GetConsolidatedCVars();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	for (const TPair<FString, FString>& Pair : ConsolidatedCvars)
	{
		if (!FuncRef(Pair.Key, Pair.Value))
		{
			return;
		}
	}	
}

#endif


#if ALLOW_OTHER_PLATFORM_CONFIG

void UDeviceProfile::ExpandDeviceProfileCVars()
{
	// VisitPlatformCVarsForEmulation can't access Scalability.h, so make sure it doesn't change away from 3, or if it does, to fix up the hardcoded number
	static_assert(Scalability::DefaultQualityLevel == 3, "If this trips, update this and IConsoleManager::VisitPlatformCVarsForEmulation with the new value!");

	IConsoleManager::VisitPlatformCVarsForEmulation(*DeviceType, GetName(),
		[this](const FString& CVarName, const FString& CVarValue, EConsoleVariableFlags SetByAndPreview)
		{
			// don't add scalabiliy groups to the expanded, but do add them to the preview set (this is to maintain same 
			// functionality for GetAllExpandedCVars(), but allow to see what Preview will set the SGs to in the SetByPreview mode)
			if (!CVarName.StartsWith(TEXT("sg.")))
			{
				AllExpandedCVars.Add(CVarName, CVarValue);
			}

			if (SetByAndPreview & EConsoleVariableFlags::ECVF_Preview)
			{
				AllPreviewCVars.Add(CVarName, CVarValue);
			}
		});
}

const TMap<FString, FString>& UDeviceProfile::GetAllExpandedCVars()
{
	// expand on first use
	if (AllExpandedCVars.Num() == 0)
	{
		ExpandDeviceProfileCVars();
	}

	return AllExpandedCVars;
}

const TMap<FString, FString>& UDeviceProfile::GetAllPreviewCVars()
{
	// expand on first use
	if (AllPreviewCVars.Num() == 0)
	{
		ExpandDeviceProfileCVars();
	}

	return AllPreviewCVars;
}

void UDeviceProfile::ClearAllExpandedCVars()
{
	AllExpandedCVars.Empty();
	AllPreviewCVars.Empty();
}

void UDeviceProfile::SetPreviewMemorySizeBucket(EPlatformMemorySizeBucket PreviewMemorySizeBucketIn)
{ 
	if (PreviewMemorySizeBucket != PreviewMemorySizeBucketIn)
	{
		PreviewMemorySizeBucket = PreviewMemorySizeBucketIn;
		// If this changes then any cached cvars are likely to be invalid too.
		ClearAllExpandedCVars();
	}
}

EPlatformMemorySizeBucket UDeviceProfile::GetPreviewMemorySizeBucket() const
{ 
	return PreviewMemorySizeBucket;
}

#endif
