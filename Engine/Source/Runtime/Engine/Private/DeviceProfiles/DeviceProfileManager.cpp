// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceProfiles/DeviceProfileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/ConfigUtilities.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "Misc/DelayedAutoRegister.h"
#include "UObject/Package.h"
#include "SceneManagement.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "UObject/UnrealType.h"
#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif
#include "IDeviceProfileSelectorModule.h"

#include "DeviceProfiles/DeviceProfileFragment.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeviceProfileManager)


DEFINE_LOG_CATEGORY_STATIC(LogDeviceProfileManager, Log, All);

static TAutoConsoleVariable<FString> CVarDeviceProfileOverride(
	TEXT("dp.Override"),
	TEXT(""),
	TEXT("DeviceProfile override - setting this will use the named DP as the active DP. In addition, it will restore any\n")
	TEXT(" previous overrides before setting (does a dp.OverridePop before setting after the first time).\n")
	TEXT(" The commandline -dp option will override this on startup, but not when setting this at runtime\n"),
	ECVF_Default);

void OnCVarDeviceProfileLogActiveDeviceProfile(IConsoleVariable*)
{
	UDeviceProfileManager& DPManager = UDeviceProfileManager::Get();
	UE_LOGF(LogDeviceProfileManager, Log, "ActiveDeviceProfileName= %ls", *DPManager.GetActiveDeviceProfileName());

	TArray<TTuple<FName, FString>> DPChain = DPManager.GetDPChain(DPManager.GetActiveDeviceProfileName(), GConfig);

	for (const auto& DP : DPChain)
	{
		const FName& DPName = DP.Get<0>();
		UE_LOGF(LogDeviceProfileManager, Log, "   %ls", *DPName.ToString());
	}
}

static TAutoConsoleVariable<bool> CVarDeviceProfileLogActiveDeviceProfile(
	TEXT("dp.LogActiveDeviceProfile"),
	false,
	TEXT("Logs the result of GetActiveDeviceProfileName and GetDPChain"),
	FConsoleVariableDelegate::CreateStatic(OnCVarDeviceProfileLogActiveDeviceProfile),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarAllowScalabilityGroupsToChangeAtRuntime(
	TEXT("dp.AllowScalabilityGroupsToChangeAtRuntime"),
	0,
	TEXT("If true, device profile scalability bucket cvars will be set with scalability")
	TEXT("priority which allows them to be changed at runtime. Off by default."),
	ECVF_Default);

TMap<FString, FString> UDeviceProfileManager::DeviceProfileScalabilityCVars;
TMap<FString, FString> UDeviceProfileManager::PreviewDeviceProfileScalabilityCVars;

FString UDeviceProfileManager::BackupSuffix = TEXT("_Backup");
TMap<FString, FPushedCVarSetting> UDeviceProfileManager::PushedSettings;
TArray<FSelectedFragmentProperties> UDeviceProfileManager::PlatformFragmentsSelected;

UDeviceProfileManager* UDeviceProfileManager::DeviceProfileManagerSingleton = nullptr;

// when objects are ready, we can create the singleton properly, this makes it as early as possible, just in case other Object creation
// didnt't create it along the way
FDelayedAutoRegisterHelper GDPManagerSingletonHelper(EDelayedRegisterRunPhase::ObjectSystemReady, [] { UDeviceProfileManager::Get(); });

static int32 CVarSetMode()
{
	static int32 Mode = -1;
	if (Mode == -1)
	{
		Mode = 0;
		// set the CVarSetMode from commandline
		FParse::Value(FCommandLine::Get(), TEXT("DPApplyMode="), Mode);
	}
	return Mode;
}

UDeviceProfileManager& UDeviceProfileManager::Get(bool bFromPostCDOContruct)
{
	if (DeviceProfileManagerSingleton == nullptr)
	{
		UE_SCOPED_ENGINE_ACTIVITY("Loading Device Profiles");
		static bool bEntered = false;
		if (bEntered && bFromPostCDOContruct)
		{
			return *(UDeviceProfileManager*)0x3; // we know that the return value is never used, linux hates null here, which would be less weird. 
		}
		bEntered = true;
		DeviceProfileManagerSingleton = NewObject<UDeviceProfileManager>();

		DeviceProfileManagerSingleton->AddToRoot();
		DeviceProfileManagerSingleton->LoadProfiles();

		// always start with an active profile, even if we create it on the spot
		UDeviceProfile* ActiveProfile = DeviceProfileManagerSingleton->FindProfile(GetPlatformDeviceProfileName());
		DeviceProfileManagerSingleton->SetActiveDeviceProfile(ActiveProfile);

		// now we allow the cvar changes to be acknowledged
		CVarDeviceProfileOverride.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
		{
			UDeviceProfileManager::Get().HandleDeviceProfileOverrideChange();
		}));

		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("dp.Override.Restore"),
			TEXT("Restores any cvars set by dp.Override to their previous value"),
			FConsoleCommandDelegate::CreateLambda([]()
			{
				UDeviceProfileManager::Get().RestoreDefaultDeviceProfile();
			}),
			ECVF_Default
		);

		InitializeSharedSamplerStates();

#if ALLOW_OTHER_PLATFORM_CONFIG
		FCoreDelegates::GatherDeviceProfileCVars.BindLambda([](const FString& DeviceProfileName) { return UDeviceProfileManager::Get().GatherDeviceProfileCVars(DeviceProfileName, EDeviceProfileMode::DPM_CacheValues); });
#endif

		// let any other code that needs the DPManager to run now
		FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::DeviceProfileManagerReady);
		
		// when we load/unload dynamic configs, we may need to propagate those changes to the active DP
		void UpdateCVarsAfterDynamicConfig(const FConfigModificationTracker* ChangeTracker);
		UE::DynamicConfig::UpdateCVarsAndDeviceProfiles.AddStatic(UpdateCVarsAfterDynamicConfig);
	}
	return *DeviceProfileManagerSingleton;
}

TArray<TTuple<FName, FString>> UDeviceProfileManager::GetDPChain(const FString& DeviceProfileName, FConfigCacheIni* ConfigSystem)
{
	static FString DPSectionSuffix = *FString::Printf(TEXT(" %s"), *UDeviceProfile::StaticClass()->GetName());
	static FString FragmentSectionSuffix = *FString::Printf(TEXT(" %s"), *UDeviceProfileFragment::StaticClass()->GetName());

	// @todo fragments, before and after

	// For each device profile, starting with the selected and working our way up the BaseProfileName tree,
	FString BaseDeviceProfileName = DeviceProfileName;
	bool bReachedEndOfTree = BaseDeviceProfileName.IsEmpty();
	TArray<FString> ParentChain;
	while (bReachedEndOfTree == false)
	{
		FString CurrentSectionName = BaseDeviceProfileName + DPSectionSuffix;

		ParentChain.Add(BaseDeviceProfileName);

		// check if there is a section named for the DeviceProfile
		const FConfigSection* CurrentSection = ConfigSystem->GetSection(*CurrentSectionName, false, GDeviceProfilesIni);
		if (CurrentSection != nullptr)
		{
			// Get the next device profile name, to look for CVars in, along the tree
			FString NextBaseDeviceProfileName;
			ConfigSystem->GetString(*CurrentSectionName, TEXT("BaseProfileName"), NextBaseDeviceProfileName, GDeviceProfilesIni);
			BaseDeviceProfileName = NextBaseDeviceProfileName;
		}

		// Check if we have inevitably reached the end of the device profile tree.
		bReachedEndOfTree = CurrentSection == nullptr || BaseDeviceProfileName.IsEmpty();
	}

	TArray<TTuple<FName, FString>> DPChain;

	// now walk up the chain, and insert fragments for each level
	for (int Index = ParentChain.Num() - 1; Index >= 0; Index--)
	{
		const FString& DPName = ParentChain[Index];
		FString CurrentSectionName = DPName + DPSectionSuffix;

		// includes of this DP
		TArray<FString> PreFragmentIncludes, PostFragmentIncludes;
		ConfigSystem->GetArray(*CurrentSectionName, TEXT("FragmentIncludes"), PreFragmentIncludes, GDeviceProfilesIni);
		ConfigSystem->GetArray(*CurrentSectionName, TEXT("OverrideFragmentIncludes"), PostFragmentIncludes, GDeviceProfilesIni);

		for (const FString& PreFragmentName : PreFragmentIncludes)
		{
			DPChain.Emplace(*PreFragmentName, PreFragmentName + FragmentSectionSuffix);
		}
		DPChain.Emplace(*DPName, CurrentSectionName);
		for (const FString& PostFragmentName : PostFragmentIncludes)
		{
			DPChain.Emplace(*PostFragmentName, PostFragmentName + FragmentSectionSuffix);
		}
	}

	// get fragments then append in reverse order
	TArray<FSelectedFragmentProperties> DynamicFragments = FindMatchingFragments(DeviceProfileName, ConfigSystem);
	for (int Index = DynamicFragments.Num() - 1; Index >= 0; Index--)
	{
		// @todo check benabled?
		DPChain.Emplace(*DynamicFragments[Index].Fragment, DynamicFragments[Index].Fragment + FragmentSectionSuffix);
	}

	return DPChain;
}

void UpdateCVarsAfterDynamicConfig(const FConfigModificationTracker* ChangeTracker)
{
	// reload objects that had their configs changed
	for (const auto& SectionPair : ChangeTracker->CVars)
	{
		const FConfigModificationTracker::FCVarTracker& CVars = SectionPair.Value;
		EConsoleVariableFlags Priority = (EConsoleVariableFlags)CVars.CVarPriority;

		// now walk over the updated cvars and set them based on the priority
		for (const auto& TagPair : CVars.CVarEntriesPerBranchPerTag)
		{
			for (const auto& BranchPair : TagPair.Value)
			{
				const FConfigSection& Section = BranchPair.Value;

				for (const auto& CVarPair : Section)
				{
					FString KeyString = CVarPair.Key.ToString();

					if (KeyString.StartsWith(TEXT("sg.")))
					{
						UE::ConfigUtilities::ApplyCVarSettingsFromIni(*FString::Printf(TEXT("%s@%s"), *KeyString.Mid(3), *CVarPair.Value.GetValue()), *GScalabilityIni, Priority, false, TagPair.Key);
					}
					else
					{
						UE::ConfigUtilities::OnSetCVarFromIniEntry(TEXT("DynamicLayer"), *CVarPair.Key.ToString(), *CVarPair.Value.GetValue(), Priority, false, false, TagPair.Key);
					}
				}
			}
		}
	}

	const TSet<FString>* DeviceProfileSections = ChangeTracker->ModifiedSectionsPerBranch.Find(*GDeviceProfilesIni);
	if (DeviceProfileSections != nullptr)
	{
		if (CVarSetMode() > 0)
		{
			UDeviceProfileManager& DPManager = UDeviceProfileManager::Get();
			IConsoleManager& ConsoleManager = IConsoleManager::Get();

			// get the set of sections the active DP cares about
			TArray<TTuple<FName, FString>> DPChain = UDeviceProfileManager::Get().GetDPChain(DPManager.GetActiveDeviceProfileName(), GConfig);

			for (const auto& DP : DPChain)
			{
				// if we updated this DP, then refresh it
				const FName& DPName = DP.Get<0>();
				const FString& DPSectionName = DP.Get<1>();
				if (DeviceProfileSections->Contains(DPSectionName))
				{
					// mode 1 we wipe everything then start over instead of updating in place
					if (CVarSetMode() == 1)
					{
						ConsoleManager.UnsetAllConsoleVariablesWithTag(DP.Get<0>());
					}

					const bool bIsFragment = DPSectionName.EndsWith(UDeviceProfileFragment::StaticClass()->GetName());
					const UDeviceProfileManager::EDeviceProfileMode DeviceProfileMode =
						bIsFragment ? UDeviceProfileManager::EDeviceProfileMode::DPM_SetCVarsForSingleFragment : UDeviceProfileManager::EDeviceProfileMode::DPM_SetCVarsForSingleDP;

					// get just this DP's cvars, and then update in place
					TMap<FName, FString> DeviceProfileCVars = DPManager.GatherDeviceProfileCVars(DPName.ToString(), DeviceProfileMode);
					ConsoleManager.BatchUpdateTag(DPName, DeviceProfileCVars);
				}
			}
		}
		else
		{
			TSet<FString> DeviceProfilesToQuery;
			int32 DeviceProfilePatternLength = UDeviceProfile::StaticClass()->GetName().Len() + 1;
			for (const FString& Section : *DeviceProfileSections)
			{
				DeviceProfilesToQuery.Add(Section.LeftChop(DeviceProfilePatternLength));
			}

			if (UDeviceProfileManager::Get().DoActiveProfilesReference(DeviceProfilesToQuery))
			{
				UDeviceProfileManager::Get().ReapplyDeviceProfile();
			}
		}
	}

	// if we modify a scalability group section (ie EffectsQuality@2) that matches current value, re-set the group to apply the modified group
	const TSet<FString>* ScalabilitySections = ChangeTracker->ModifiedSectionsPerBranch.Find(*GScalabilityIni);
	if (ScalabilitySections != nullptr)
	{
		for (const FString& SectionName : *ScalabilitySections)
		{
			FString Group, Level;
			if (SectionName.Split(TEXT("@"), &Group, &Level))
			{
				FString SGName = FString::Printf(TEXT("sg.%s"), *Group);
				IConsoleVariable* SGVar = IConsoleManager::Get().FindConsoleVariable(*SGName);
				if (SGVar != nullptr)
				{
					if (SGVar->GetString() == Level)
					{
						SGVar->SetWithCurrentPriority(*Level);
					}
				}
			}
		}
	}
}

// Read the cvars from a [DeviceProfileFragment] section.
static bool GetFragmentCVars(const FString& FragmentName, const FString& CVarArrayName, TArray<FString>& FragmentCVarsINOUT, FConfigCacheIni* ConfigSystem)
{
	FString FragmentSectionName = FString::Printf(TEXT("%s %s"), *FragmentName, *UDeviceProfileFragment::StaticClass()->GetName());
	if (ConfigSystem->DoesSectionExist(*FragmentSectionName, GDeviceProfilesIni))
	{
		TArray<FString> FragmentCVars;
		ConfigSystem->GetArray(*FragmentSectionName, *CVarArrayName, FragmentCVars, GDeviceProfilesIni);
		UE_CLOGF(FragmentCVars.Num() > 0, LogInit, Log, "Including %ls from fragment: %ls", *CVarArrayName, *FragmentName);
		FragmentCVarsINOUT += FragmentCVars;
	}
	else
	{
		UE_LOGF(LogInit, Error, "Could not find device profile fragment %ls.", *FragmentName);
		return false;
	}
	return true;
}

// Read the requested fragment from within the +FragmentIncludes= array of a DP. When bForAppending is set to true, we'll use OverrideFragmentIncludes instead.
static void GetCVarsFromDPFragmentIncludes(const FString& CurrentSectionName, const FString& CVarArrayName, TArray<FString>& FragmentCVarsINOUT, FConfigCacheIni* ConfigSystem, bool bForAppending)
{
	const FString FragmentIncludes = bForAppending ? TEXT("OverrideFragmentIncludes") : TEXT("FragmentIncludes");
	TArray<FString> FragmentIncludeArray;
	ConfigSystem->GetArray(*CurrentSectionName, *FragmentIncludes, FragmentIncludeArray, GDeviceProfilesIni);

	for(const FString& FragmentInclude : FragmentIncludeArray)
	{
		GetFragmentCVars(FragmentInclude, CVarArrayName, FragmentCVarsINOUT, ConfigSystem);
	}
}


static void ExpandScalabilityCVar(FConfigCacheIni* ConfigSystem, const FString& CVarKey, const FString CVarValue, TMap<FString, FString>& ExpandedCVars, bool bOverwriteExistingValue)
{
	// load scalability settings directly from ini instead of using scalability system, so as not to inadvertantly mess anything up
	// if the DP had sg.ViewDistanceQuality=3, we would read [ViewDistanceQuality@3]
	FString SectionName = FString::Printf(TEXT("%s@%s"), *CVarKey.Mid(3), *CVarValue);
	// walk over the scalability section and add them in, unless already done
	const FConfigSection* ScalabilitySection = ConfigSystem->GetSection(*SectionName, false, GScalabilityIni);
	if (ScalabilitySection != nullptr)
	{
		for (const auto& Pair : *ScalabilitySection)
		{
			FString ScalabilityKey = Pair.Key.ToString();
			if (bOverwriteExistingValue || !ExpandedCVars.Contains(ScalabilityKey))
			{
				ExpandedCVars.Add(ScalabilityKey, Pair.Value.GetValue());
			}
		}
	}
}

TMap<FName, FString> UDeviceProfileManager::GatherDeviceProfileCVars(const FString& DeviceProfileName, EDeviceProfileMode GatherMode)
{
	FConfigCacheIni* ConfigSystem = GConfig;
	// build up cvars into a map
	TMap<FName, FString> DeviceProfileCVars;
	TArray<FSelectedFragmentProperties> FragmentsSelected;
	TSet<FString> FragmentCVarKeys;
	TArray<FString> SelectedFragmentCVars;
	bool bSingleSectionMode = GatherMode == EDeviceProfileMode::DPM_SetCVarsForSingleDP || GatherMode == EDeviceProfileMode::DPM_SetCVarsForSingleFragment;


	EPlatformMemorySizeBucket MemBucket = FPlatformMemory::GetMemorySizeBucket();
	// if caching (for another platform), then we use the DP's PreviewMemoryBucket, instead of querying for it
	if (GatherMode == EDeviceProfileMode::DPM_CacheValues || GatherMode == EDeviceProfileMode::DPM_CacheValuesIgnoreMatchingRules )
	{
#if ALLOW_OTHER_PLATFORM_CONFIG
		// caching is not done super early, so we can assume DPs have been found now
		UDeviceProfile* Profile = UDeviceProfileManager::Get().FindProfile(DeviceProfileName, true);
		if (Profile == nullptr)
		{
			UE_LOGF(LogDeviceProfileManager, Log, "Unable to find DeviceProfile '%ls' for gathering cvars for caching. Aborting....", *DeviceProfileName);
			check(Profile);
			return DeviceProfileCVars;
		}

		// use the DP's platform's configs, NOT the running platform
		ConfigSystem = FConfigCacheIni::ForPlatform(*Profile->DeviceType);

		MemBucket = Profile->GetPreviewMemorySizeBucket();
		if(GatherMode == EDeviceProfileMode::DPM_CacheValues)
		{
			FragmentsSelected = FindMatchingFragments(DeviceProfileName, ConfigSystem);
		}
#else
		checkNoEntry();
#endif
	}
	// no fragments when doing single at a time
	else if (!bSingleSectionMode)
	{
		// the very first time, this is called to set cvars, we cache the current device's fragments
		if (PlatformFragmentsSelected.Num() == 0)
		{
			PlatformFragmentsSelected = FindMatchingFragments(DeviceProfileName, GConfig);
			if(PlatformFragmentsSelected.Num())
			{
				// Store the fragment string:
				FString MatchedFragmentString = FragmentPropertyArrayToFragmentString(PlatformFragmentsSelected, false, false, true);
				FGenericCrashContext::SetEngineData(TEXT("DeviceProfile.MatchedFragmentsSorted"), MatchedFragmentString);
			}
		}
		FragmentsSelected = PlatformFragmentsSelected;
	}


	// directly look in the .ini instead of using FindDeviceProfile - not all of the parent DPs may be in existence, but we still need to go up the chain
	// this will cache the DP names we know about
	TArray<FString> AvailableProfiles;
	ConfigSystem->GetSectionNames(GDeviceProfilesIni, AvailableProfiles);
	AvailableProfiles.Remove(TEXT("DeviceProfiles"));


	if (!bSingleSectionMode)
	{
		// Here we gather the cvars from selected fragments in reverse order
		for (int i = FragmentsSelected.Num() - 1; i >= 0; i--)
		{
			const FSelectedFragmentProperties& SelectedFragment = FragmentsSelected[i];
			if (SelectedFragment.bEnabled)
			{
				TArray<FString> FragmentCVars;
				GetFragmentCVars(*SelectedFragment.Fragment, TEXT("CVars"), FragmentCVars, ConfigSystem);
				for (int j = FragmentCVars.Num() - 1; j >= 0; j--)
				{
					const FString& FragCVar = FragmentCVars[j];

					FString CVarKey, CVarValue;
					if (FragCVar.Split(TEXT("="), &CVarKey, &CVarValue))
					{
						if (!FragmentCVarKeys.Find(CVarKey))
						{
							FragmentCVarKeys.Add(CVarKey);
							SelectedFragmentCVars.Add(FragCVar);
						}
					}
				}
			}
		}
	}

	const TCHAR* SectionSuffix = GatherMode == EDeviceProfileMode::DPM_SetCVarsForSingleFragment ? TEXT(" DeviceProfileFragment") : TEXT(" DeviceProfile");

	// For each device profile, starting with the selected and working our way up the BaseProfileName tree,
	FString BaseDeviceProfileName = DeviceProfileName;
	bool bReachedEndOfTree = BaseDeviceProfileName.IsEmpty();
	while (bReachedEndOfTree == false)
	{
		FString CurrentSectionName = BaseDeviceProfileName + SectionSuffix;

		// check if there is a section named for the DeviceProfile
		const FConfigSection* CurrentSection = ConfigSystem->GetSection(*CurrentSectionName, false, GDeviceProfilesIni);
		if (CurrentSection != nullptr)
		{
			// put this up in some shared code somewhere in FGenericPlatformMemory
			const TCHAR* BucketNames[] = {
				TEXT("_Largest"),
				TEXT("_Larger"),
				TEXT("_Default"),
				TEXT("_Smaller"),
				TEXT("_Smallest"),
				TEXT("_Tiniest"),
			};

			for (int Pass = 0; Pass < 2; Pass++)
			{
				// apply the current memory bucket CVars in Pass 0, regular CVars in pass 1 (anything set in Pass 0 won't be set in pass 1)
				FString ArrayName = "CVars";
				if (Pass == 0)
				{
					ArrayName += BucketNames[(int32)MemBucket];
				}

				TArray< FString > CurrentProfilesCVars, FragmentCVars, OverrideFragmentCVars;
				if (!bSingleSectionMode)
				{
					GetCVarsFromDPFragmentIncludes(*CurrentSectionName, *ArrayName, FragmentCVars, ConfigSystem, false);
					GetCVarsFromDPFragmentIncludes(*CurrentSectionName, *ArrayName, OverrideFragmentCVars, ConfigSystem, true);
				}
				CurrentSection->MultiFind(*ArrayName, CurrentProfilesCVars, true);

				if (FragmentCVars.Num())
				{
					// Prepend fragments to CurrentProfilesCVars, fragment cvars should be first so the DP's cvars take priority.
					Swap(CurrentProfilesCVars, FragmentCVars);
					CurrentProfilesCVars += FragmentCVars;
				}

				CurrentProfilesCVars += OverrideFragmentCVars;

				// now add the selected fragments at the end so these override the DP.
				CurrentProfilesCVars += SelectedFragmentCVars;
				SelectedFragmentCVars.Empty();

				// Iterate over the profile and make sure we do not have duplicate CVars
				{
					TMap< FString, FString > ValidCVars;
					for (TArray< FString >::TConstIterator CVarIt(CurrentProfilesCVars); CVarIt; ++CVarIt)
					{
						FString CVarKey, CVarValue;
						if ((*CVarIt).Split(TEXT("="), &CVarKey, &CVarValue))
						{
							UE_LOGF(LogDeviceProfileManager, VeryVerbose, "ValidCVars: %ls=%ls", *CVarKey, *CVarValue);
							ValidCVars.Add(CVarKey, CVarValue);
						}
					}

					// Empty the current list, and replace with the processed CVars. This removes duplicates
					CurrentProfilesCVars.Empty();

					for (TMap< FString, FString >::TConstIterator ProcessedCVarIt(ValidCVars); ProcessedCVarIt; ++ProcessedCVarIt)
					{
						UE_LOGF(LogDeviceProfileManager, VeryVerbose, "CurrentProfilesCVars: %ls=%ls", *ProcessedCVarIt.Key(), *ProcessedCVarIt.Value());
						CurrentProfilesCVars.Add(FString::Printf(TEXT("%s=%s"), *ProcessedCVarIt.Key(), *ProcessedCVarIt.Value()));
					}

				}

				// Iterate over this profiles cvars and set them if they haven't been already.
				for (TArray< FString >::TConstIterator CVarIt(CurrentProfilesCVars); CVarIt; ++CVarIt)
				{
					FString CVarKey, CVarValue;
					if ((*CVarIt).Split(TEXT("="), &CVarKey, &CVarValue))
					{
						FName CVarKeyName(*CVarKey);
						if (!DeviceProfileCVars.Find(CVarKeyName))
						{
							UE_LOGF(LogDeviceProfileManager, VeryVerbose, "DeviceProfileCVars: %ls=%ls", *CVarKey, *CVarValue);
							DeviceProfileCVars.Add(CVarKeyName, CVarValue);
						}
					}
				}
			}

			// Get the next device profile name, to look for CVars in, along the tree
			FString NextBaseDeviceProfileName;
			if (!bSingleSectionMode && ConfigSystem->GetString(*CurrentSectionName, TEXT("BaseProfileName"), NextBaseDeviceProfileName, GDeviceProfilesIni))
			{
				BaseDeviceProfileName = NextBaseDeviceProfileName;
				UE_LOGF(LogDeviceProfileManager, Log, "Going up to parent DeviceProfile [%ls]", *BaseDeviceProfileName);
			}
			else
			{
				BaseDeviceProfileName.Empty();
			}
		}

		// Check if we have inevitably reached the end of the device profile tree.
		bReachedEndOfTree = bSingleSectionMode || CurrentSection == nullptr || BaseDeviceProfileName.IsEmpty();
	}

	return DeviceProfileCVars;
}

TMap<FName, TSet<FString>> UDeviceProfileManager::GetAllReferencedDeviceProfileCVars(UDeviceProfile* DeviceProfile)
{
	FConfigCacheIni* ConfigSystem = GConfig;

#if ALLOW_OTHER_PLATFORM_CONFIG
	// use the DP's platform's configs, NOT the running platform
	ConfigSystem = FConfigCacheIni::ForPlatform(*DeviceProfile->DeviceType);
#endif

	TMap<FName, FString> DeviceProfileCVars = GatherDeviceProfileCVars(DeviceProfile->GetName(), EDeviceProfileMode::DPM_CacheValuesIgnoreMatchingRules);

	// gather all referenced fragments, note that this just a dumb traverse of the the matched rules so it may contain
	// fragments that a device cannot ultimately select.
	TArray<FString> AllReferencedMatchedFragments = FindAllReferencedFragmentsFromMatchedRules(DeviceProfile->GetName(), ConfigSystem);

	TMap<FName, TSet<FString>> AllCVarsAndValues;

	for (const auto& Pair : DeviceProfileCVars)
	{
		AllCVarsAndValues.FindOrAdd(Pair.Key).Add(Pair.Value);
	}

	for (const FString& Fragment : AllReferencedMatchedFragments)
	{
		TArray<FString> FragmentCVars;
		GetFragmentCVars(Fragment, TEXT("CVars"), FragmentCVars, ConfigSystem);
		for (const FString& FragCVar : FragmentCVars)
		{
			FString CVarKey, CVarValue;
			if (FragCVar.Split(TEXT("="), &CVarKey, &CVarValue))
			{
				if (CVarKey.StartsWith(TEXT("sg.")))
				{
					TMap<FString, FString> ScalabilityCVars;
					ExpandScalabilityCVar(ConfigSystem, CVarKey, CVarValue, ScalabilityCVars, true);
					for (const auto& ScalabilityPair : ScalabilityCVars)
					{
						AllCVarsAndValues.FindOrAdd(*ScalabilityPair.Key).Add(ScalabilityPair.Value);
					}
				}
				else
				{
					AllCVarsAndValues.FindOrAdd(FName(CVarKey)).Add(CVarValue);
				}
			}
		}
	}

	return AllCVarsAndValues;
}

void UDeviceProfileManager::SetDeviceProfileCVars(const FString& DeviceProfileName)
{
	if (CVarSetMode() > 0)
	{
		FConfigCacheIni* ConfigSystem = GConfig;
		TArray<TTuple<FName, FString>> DPChain = UDeviceProfileManager::Get().GetDPChain(DeviceProfileName, ConfigSystem);

		// now that we have the ordered chain, walk it and set cvars
		for (const TTuple<FName, FString>& DP : DPChain)
		{
			FName DPName = DP.Get<0>();
			const FString& SectionName = DP.Get<1>();
			EDeviceProfileMode GatherMode = EDeviceProfileMode::DPM_SetCVarsForSingleDP;
			if (SectionName.EndsWith(UDeviceProfileFragment::StaticClass()->GetName()))
			{
				GatherMode = EDeviceProfileMode::DPM_SetCVarsForSingleFragment;
			}

			// get the cvars for the DP or the Fragment
			TMap<FName, FString> DeviceProfileCVars = GatherDeviceProfileCVars(DPName.ToString(), GatherMode);

			for (const TPair<FName, FString>& Pair : DeviceProfileCVars)
			{
				const FString CVarKey = Pair.Key.ToString();
				const FString& CVarValue = Pair.Value;

				// Cache any scalability related cvars so we can conveniently reapply them later as a way to reset the device defaults
				bool bIsScalabilityBucket = CVarKey.StartsWith(TEXT("sg."));
				//if (bIsScalabilityBucket && CVarAllowScalabilityGroupsToChangeAtRuntime.GetValueOnGameThread() > 0)
				//{
				//	DeviceProfileScalabilityCVars.Add(*CVarKey, *CVarValue);
				//}

				// Set by scalability or DP, depending
				uint32 CVarPriority = bIsScalabilityBucket ? ECVF_SetByScalability : ECVF_SetByDeviceProfile;
				UE::ConfigUtilities::OnSetCVarFromIniEntry(*GDeviceProfilesIni, *CVarKey, *CVarValue, CVarPriority, false, true, DPName);
			}
		}
	}
	else
	{
		// walk over the parent chain, gathering the cvars this DP inherits and contains
		TMap<FName, FString> DeviceProfileCVars = GatherDeviceProfileCVars(DeviceProfileName, EDeviceProfileMode::DPM_SetCVars);

		// reset some global state
		DeviceProfileScalabilityCVars.Empty();

		// we should have always popped away old values by the time we get here
		check(PushedSettings.Num() == 0);

		// Preload a cvar we rely on in the loop below
		if (const FConfigSection* Section = GConfig->GetSection(TEXT("ConsoleVariables"), false, *GEngineIni))
		{
			static FName AllowScalabilityAtRuntimeName = "dp.AllowScalabilityGroupsToChangeAtRuntime";
			if (const FConfigValue* Value = Section->Find(AllowScalabilityAtRuntimeName))
			{
				const FString& KeyString = AllowScalabilityAtRuntimeName.ToString();
				const FString& ValueString = Value->GetValue();
				UE::ConfigUtilities::OnSetCVarFromIniEntry(*GEngineIni, *KeyString, *ValueString, ECVF_SetBySystemSettingsIni);
			}
		}

		// for each DP cvar - push the old value to restore, cache ScalabiliutyGroups, and then set the values
		for (TPair<FName, FString>& Pair : DeviceProfileCVars)
		{
			const FString CVarKey = Pair.Key.ToString();
			const FString& CVarValue = Pair.Value;

			// get the actual cvar object
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarKey);
			if (CVar)
			{
				// remember the previous value and priority
				FPushedCVarSetting OldSetting = FPushedCVarSetting(CVar->GetString(), CVar->GetFlags());
				PushedSettings.Add(CVarKey, OldSetting);

				// indicate we are pushing, not setting
				UE_LOGF(LogDeviceProfileManager, Log, "Pushing Device Profile CVar: [[%ls:%ls -> %ls]]", *CVarKey, *OldSetting.Value, *CVarValue);
			}
			else
			{
				UE_LOGF(LogDeviceProfileManager, Log, "Creating unregistered Device Profile CVar: [[%ls:%ls]]", *CVarKey, *Pair.Value);
			}

			// Cache any scalability related cvars so we can conveniently reapply them later as a way to reset the device defaults
			bool bIsScalabilityBucket = CVarKey.StartsWith(TEXT("sg."));
			if (bIsScalabilityBucket && CVarAllowScalabilityGroupsToChangeAtRuntime.GetValueOnGameThread() > 0)
			{
				DeviceProfileScalabilityCVars.Add(*CVarKey, *CVarValue);
			}

			// Set by scalability or DP, depending
			uint32 CVarPriority = bIsScalabilityBucket ? ECVF_SetByScalability : ECVF_SetByDeviceProfile;
			UE::ConfigUtilities::OnSetCVarFromIniEntry(*GDeviceProfilesIni, *CVarKey, *CVarValue, CVarPriority, false, true);
		}
	}

	// now allow for ovverrides in !SHIPPING
#if !UE_BUILD_SHIPPING
	// allow ConfigRules to override cvars first
	const TMap<FString, FString>& ConfigRules = FPlatformMisc::GetConfigRuleVars();
	for (const TPair<FString, FString>& Pair : ConfigRules)
	{
		FString Key = Pair.Key;
		if (Key.StartsWith("cvar_"))
		{
			FString CVarKey = Key.RightChop(5);
			FString CVarValue = Pair.Value;

			UE_LOGF(LogDeviceProfileManager, Log, "Setting ConfigRules Device Profile CVar: [[%ls:%ls]]", *CVarKey, *CVarValue);

			// set it and remember it
			UE::ConfigUtilities::OnSetCVarFromIniEntry(*GDeviceProfilesIni, *CVarKey, *CVarValue, ECVF_SetByDeviceProfile);
		}
	}

	// pre-apply any -dpcvars= items, so that they override anything in the DPs
	// Search for all occurrences of dpcvars and dpcvar on the command line.
	static const TCHAR* DPCVarTags[]{ TEXT("DPCVars="), TEXT("DPCVar="), TEXT("ForceDPCVars=") };
	static const EConsoleVariableFlags DPCVarPri[] = { ECVF_SetByDeviceProfile, ECVF_SetByDeviceProfile, ECVF_SetByCommandline };
	static_assert(UE_ARRAY_COUNT(DPCVarTags) == UE_ARRAY_COUNT(DPCVarPri));

	for (int i = 0; i<UE_ARRAY_COUNT(DPCVarTags) ; i++)
	{
		const EConsoleVariableFlags RequestedPri = DPCVarPri[i];
		const TCHAR* Tag = DPCVarTags[i];
		const TCHAR* RequestedPriDesc = GetConsoleVariableSetByName(RequestedPri);
		FString DPCVarString;
		for (const TCHAR* Cursor = FCommandLine::Get(); (Cursor != nullptr) && FParse::Value(Cursor, Tag, DPCVarString, false, &Cursor);)
		{
			// look over a list of cvars
			TArray<FString> DPCVars;
			DPCVarString.ParseIntoArray(DPCVars, TEXT(","), true);
			for (const FString& DPCVar : DPCVars)
			{
				// split up each Key=Value pair
				FString CVarKey, CVarValue;
				if (DPCVar.Split(TEXT("="), &CVarKey, &CVarValue))
				{
					UE_LOGF(LogDeviceProfileManager, Log, "Setting CommandLine Device Profile CVar: [[%ls:%ls]]", *CVarKey, *CVarValue);

					// set it and remember it (no thanks, Ron Popeil)
					UE::ConfigUtilities::OnSetCVarFromIniEntry(*GDeviceProfilesIni, *CVarKey, *CVarValue, RequestedPri);

					// Log if the change would not applied.
					if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarKey))
					{
						const EConsoleVariableFlags ExistingPri = (EConsoleVariableFlags)(CVar->GetFlags() & ECVF_SetByMask);
						UE_CLOG(RequestedPri < ExistingPri, LogDeviceProfileManager, Warning, TEXT("-%s%s=%s requested priority is too low (%s < %s), value remains %s"), Tag, *CVarKey, *CVarValue, GetConsoleVariableSetByName(RequestedPri), GetConsoleVariableSetByName(ExistingPri), *CVar->GetString() );
					}
				}
			}
		}
	}
#endif
}

/**
* Set the cvar state to PushedSettings.
*/
static void RestorePushedState(TMap<FString, FPushedCVarSetting>& PushedSettings)
{
	if (CVarSetMode() > 0)
	{
		IConsoleManager& ConsoleManager = IConsoleManager::Get();
		TArray<TTuple<FName, FString>> DPChain = UDeviceProfileManager::Get().GetDPChain(UDeviceProfileManager::Get().GetActiveDeviceProfileName(), GConfig);
		for (const auto& DP : DPChain)
		{
			ConsoleManager.UnsetAllConsoleVariablesWithTag(DP.Get<0>());
		}
	}
	else
	{
		// restore pushed settings
		for (TMap<FString, FPushedCVarSetting>::TIterator It(PushedSettings); It; ++It)
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*It.Key());
			if (CVar)
			{
				const FPushedCVarSetting& Setting = It.Value();

				// Check if the priority has increased since we pushed it and is now higher than SetByDeviceProfile
				EConsoleVariableFlags CurrentPriority = EConsoleVariableFlags(int32(CVar->GetFlags()) & (int32)ECVF_SetByMask);
				if (CurrentPriority > ECVF_SetByDeviceProfile && CurrentPriority > Setting.SetBy )
				{
					UE_LOGF(LogDeviceProfileManager, Warning, "Popping Device Profile CVar skipped because priority has been overridden to higher than ECVF_SetByDeviceProfile since the last push [[%ls:%ls]]", *It.Key(), *Setting.Value);
				}
				else
				{
					// restore it!
					CVar->SetWithCurrentPriority(*Setting.Value);
					UE_LOGF(LogDeviceProfileManager, Log, "Popping Device Profile CVar: [[%ls:%ls]]", *It.Key(), *Setting.Value);
				}

			}
		}

		PushedSettings.Reset();
	}
}

const FSelectedFragmentProperties* UDeviceProfileManager::GetActiveDeviceProfileFragmentByTag(FName& FragmentTag) const
{
	for (const FSelectedFragmentProperties& SelectedFragment : PlatformFragmentsSelected)
	{
		if (SelectedFragment.Tag == FragmentTag)
		{
			return &SelectedFragment;
		}
	}
	return nullptr;
}

// enable/disable a tagged fragment.
void UDeviceProfileManager::ChangeTaggedFragmentState(FName FragmentTag, bool bNewState)
{
	for (FSelectedFragmentProperties& Fragment : PlatformFragmentsSelected)
	{
		if (Fragment.Tag == FragmentTag)
		{
			if (bNewState != Fragment.bEnabled)
			{
				UE_LOGF(LogInit, Log, "ChangeTaggedFragmentState: %ls=%d", *FragmentTag.ToString(), bNewState);
				// unset entire DP's cvar state.
				RestorePushedState(PushedSettings);
				// set the new state and reapply all fragments.
				Fragment.bEnabled = bNewState;
				SetDeviceProfileCVars(GetActiveDeviceProfileName());
			}
			break;
		}
	}
}

void UDeviceProfileManager::InitializeCVarsForActiveDeviceProfile(bool bPushSettings, bool bIsDeviceProfilePreview, bool bForceReload)
{
	FString ActiveProfileName;

	if (DeviceProfileManagerSingleton)
	{
		ActiveProfileName = DeviceProfileManagerSingleton->ActiveDeviceProfile->GetName();
		//Ensure we've loaded the device profiles for the active platform.
		//This can be needed when overriding the device profile.
		FConfigContext Context = FConfigContext::ReadIntoGConfig();
		Context.bForceReload = bForceReload;
		Context.Load(TEXT("DeviceProfiles"), GDeviceProfilesIni);
	}
	else
	{
		ActiveProfileName = GetPlatformDeviceProfileName();
	}

	UE_LOGF(LogInit, Log, "Selected Device Profile: [%ls]", *ActiveProfileName);

	SetDeviceProfileCVars(ActiveProfileName);
}


bool UDeviceProfileManager::DoActiveProfilesReference(const TSet<FString>& DeviceProfilesToQuery)
{
	TArray< FString > AvailableProfiles;
	GConfig->GetSectionNames(GDeviceProfilesIni, AvailableProfiles);

	auto DoesProfileReference = [&AvailableProfiles, GDeviceProfilesIni = GDeviceProfilesIni](const FString& SearchProfile, const TSet<FString>& InDeviceProfilesToQuery)
	{
		// For each device profile, starting with the selected and working our way up the BaseProfileName tree,
		FString BaseDeviceProfileName = SearchProfile;
		bool bReachedEndOfTree = BaseDeviceProfileName.IsEmpty();
		while (bReachedEndOfTree == false)
		{
			FString CurrentSectionName = FString::Printf(TEXT("%s %s"), *BaseDeviceProfileName, *UDeviceProfile::StaticClass()->GetName());
			bool bProfileExists = AvailableProfiles.Contains(CurrentSectionName);
			if (bProfileExists)
			{
				if (InDeviceProfilesToQuery.Contains(BaseDeviceProfileName))
				{
					return true;
				}

				// Get the next device profile name
				FString NextBaseDeviceProfileName;
				if (GConfig->GetString(*CurrentSectionName, TEXT("BaseProfileName"), NextBaseDeviceProfileName, GDeviceProfilesIni))
				{
					BaseDeviceProfileName = NextBaseDeviceProfileName;
				}
				else
				{
					BaseDeviceProfileName.Empty();
				}
			}
			bReachedEndOfTree = !bProfileExists || BaseDeviceProfileName.IsEmpty();
		}
		return false;
	};

	bool bDoActiveProfilesReferenceQuerySet = DoesProfileReference(DeviceProfileManagerSingleton->GetActiveProfile()->GetName(), DeviceProfilesToQuery);
	if (!bDoActiveProfilesReferenceQuerySet && DeviceProfileManagerSingleton->BaseDeviceProfile != nullptr)
	{
		bDoActiveProfilesReferenceQuerySet = DoesProfileReference(DeviceProfileManagerSingleton->BaseDeviceProfile->GetName(), DeviceProfilesToQuery);
	}
	return bDoActiveProfilesReferenceQuerySet;
}

void UDeviceProfileManager::ReapplyDeviceProfile(bool bForceReload, TSet<FString>* ModifiedCVars)
{	
	UDeviceProfile* OverrideProfile = DeviceProfileManagerSingleton->BaseDeviceProfile ? DeviceProfileManagerSingleton->GetActiveProfile() : nullptr;
	UDeviceProfile* BaseProfile = DeviceProfileManagerSingleton->BaseDeviceProfile ? DeviceProfileManagerSingleton->BaseDeviceProfile : DeviceProfileManagerSingleton->GetActiveProfile();

	UE_LOGF(LogDeviceProfileManager, Log, "ReapplyDeviceProfile applying profile: [%ls]", *BaseProfile->GetName());

	PreReapplyDeviceProfilesDelegate.Broadcast();

	if (ModifiedCVars != nullptr)
	{
		// make sure any modified cvar is in the pushed state because we are going to set cvars without Popping first
		if (OverrideProfile)
		{
			for (const FString& CVarName : *ModifiedCVars)
			{
				if (!PushedSettings.Contains(CVarName))
				{
					IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
					if (CVar)
					{
						FPushedCVarSetting OldSetting = FPushedCVarSetting(CVar->GetString(), CVar->GetFlags());
						PushedSettings.Add(CVarName, OldSetting);
					}
				}
			}
			SetDeviceProfileCVars(DeviceProfileManagerSingleton->GetActiveProfile()->GetName());
		}
	}
	else if (OverrideProfile)
	{
		UE_LOGF(LogDeviceProfileManager, Log, "ReapplyDeviceProfile applying override profile: [%ls]", *OverrideProfile->GetName());
		// reapply the override.
		SetOverrideDeviceProfile(OverrideProfile);
	}
	else
	{
		// reset any fragments, this will cause them to be rematched.
		PlatformFragmentsSelected.Empty();
		// restore to the pre-DP cvar state:
		RestorePushedState(PushedSettings);

		// Apply the active DP. 
		InitializeCVarsForActiveDeviceProfile(false, false, bForceReload);

		// broadcast cvar sinks now that we are done
		IConsoleManager::Get().CallAllConsoleVariableSinks();
	}

	PostReapplyDeviceProfilesDelegate.Broadcast();
}

static void TestProfileForCircularReferences(const FString& ProfileName, const FString& ParentName, const FConfigFile &PlatformConfigFile)
{
	TArray<FString> ProfileDependancies;
	ProfileDependancies.Add(ProfileName);
	FString CurrentParent = ParentName;
	while (!CurrentParent.IsEmpty())
	{
		if (ProfileDependancies.FindByPredicate([CurrentParent](const FString& InName) { return InName.Equals(CurrentParent); }))
		{
			UE_LOGF(LogDeviceProfileManager, Fatal, "Device Profile %ls has a circular dependency on %ls", *ProfileName, *CurrentParent);
		}
		else
		{
			ProfileDependancies.Add(CurrentParent);
			const FString SectionName = FString::Printf(TEXT("%s %s"), *CurrentParent, *UDeviceProfile::StaticClass()->GetName());
			CurrentParent.Reset();
			PlatformConfigFile.GetString(*SectionName, TEXT("BaseProfileName"), CurrentParent);
		}
	}
}

UDeviceProfile* UDeviceProfileManager::CreateProfile(const FString& ProfileName, const FString& ProfileType, const FString& InSpecifyParentName, const TCHAR* ConfigPlatform)
{
	UDeviceProfile* DeviceProfile = FindObject<UDeviceProfile>( GetTransientPackage(), *ProfileName );
	if (DeviceProfile == NULL)
	{
		// set the config platform if it's confidential - so that we will save to platform specific DP.ini, instead of DefaultDP.ini
		bool bNeedSaveToPlatformConfig = ConfigPlatform != nullptr && FDataDrivenPlatformInfoRegistry::GetPlatformInfo(FName(ConfigPlatform)).bIsConfidential;

		// use ConfigPlatform ini hierarchy to look in for the parent profile
		// @todo config: we could likely cache local ini files to speed this up,
		// along with the ones we load in LoadConfig
		// NOTE: This happens at runtime, so maybe only do this if !RequiresCookedData()?
		const FConfigFile* PlatformConfigFile = nullptr;
		FConfigFile LocalConfigFile;
		if (FPlatformProperties::RequiresCookedData())
		{
			PlatformConfigFile = GConfig->Find(GDeviceProfilesIni);
		}
		else
		{
			PlatformConfigFile = FConfigCacheIni::FindOrLoadPlatformConfig(LocalConfigFile, TEXT("DeviceProfiles"), ConfigPlatform);
		}
#if !UE_BUILD_SHIPPING
		if (!PlatformConfigFile->Contains(ProfileName) && !PlatformConfigFile->Contains(ProfileName + " DeviceProfile"))
		{
			// Display and not Error to allow tests to create profiles without failing
			UE_LOGF(LogDeviceProfileManager, Display, "Deviceprofile %ls not found.", *ProfileName);
		}
#endif

		// Build Parent objects first. Important for setup
		FString ParentName = InSpecifyParentName;
		if (ParentName.Len() == 0)
		{
			const FString SectionName = FString::Printf(TEXT("%s %s"), *ProfileName, *UDeviceProfile::StaticClass()->GetName());
			PlatformConfigFile->GetString(*SectionName, TEXT("BaseProfileName"), ParentName);
		}

		UDeviceProfile* ParentObject = nullptr;
		// Recursively build the parent tree
		if (ParentName.Len() > 0 && ParentName != ProfileName)
		{
			ParentObject = FindObject<UDeviceProfile>(GetTransientPackage(), *ParentName);
			if (ParentObject == nullptr)
			{
				TestProfileForCircularReferences(ProfileName, ParentName, *PlatformConfigFile);
				ParentObject = CreateProfile(ParentName, ProfileType, TEXT(""), ConfigPlatform);
			}
		}

		// Create the profile after it's parents have been created.
		DeviceProfile = NewObject<UDeviceProfile>(GetTransientPackage(), *ProfileName);
		if (ConfigPlatform != nullptr)
		{
			// if the config needs to come from a platform, set it now, then reload the config
			DeviceProfile->ConfigPlatform = ConfigPlatform;
			DeviceProfile->LoadConfig();
			
			// when we don't need to save to PlatformDP.ini, we want to save to Project/Config/DefaultDP.ini, otherwise
			// we want to save to Project/Platforms/Plat/Config/PlatDP.ini, so we keep the COnfigPlatform
			// around for saving, so it will route to the correct .ini file to save (once it's loaded, the function
			// UDeviceProfile::GetConfigOverridePlatform() is only needed when saving)
			if (!bNeedSaveToPlatformConfig)
			{
				DeviceProfile->ConfigPlatform = TEXT("");
			}
		}

		// make sure the DP has all the LODGroups it needs
		DeviceProfile->ValidateProfile();

		// if the config didn't specify a DeviceType, use the passed in one
		if (DeviceProfile->DeviceType.IsEmpty())
		{
			DeviceProfile->DeviceType = ProfileType;
		}

		// final fixups
		DeviceProfile->BaseProfileName = DeviceProfile->BaseProfileName.Len() > 0 ? DeviceProfile->BaseProfileName : ParentName;
		DeviceProfile->Parent = ParentObject;
		// the DP manager can be marked as Disregard for GC, so what it points to needs to be in the Root set
		DeviceProfile->AddToRoot();

		// Add the new profile to the accessible device profile list
		Profiles.Add( DeviceProfile );

		// Inform any listeners that the device list has changed
		ManagerUpdatedDelegate.Broadcast(); 
	}

	return DeviceProfile;
}

bool UDeviceProfileManager::HasLoadableProfileName(const FString& ProfileName, FName OptionalPlatformName)
{
	UDeviceProfile* DeviceProfile = FindObject<UDeviceProfile>(GetTransientPackage(), *ProfileName);
	if (DeviceProfile != nullptr)
	{
		return true;
	}

	FConfigCacheIni* ConfigSystem = GConfig;

	if (OptionalPlatformName != NAME_None)
	{
#if ALLOW_OTHER_PLATFORM_CONFIG
		ConfigSystem = FConfigCacheIni::ForPlatform(OptionalPlatformName);
#else

		checkf(OptionalPlatformName == FName(FPlatformProperties::IniPlatformName()), 
			TEXT("UDeviceProfileManager::HasLoadableProfileName - This platform cannot load configurations for other platforms."));
#endif
	}

	// use ConfigPlatform ini hierarchy to look in for the parent profile
	// @todo config: we could likely cache local ini files to speed this up,
	// along with the ones we load in LoadConfig
	// NOTE: This happens at runtime, so maybe only do this if !RequiresCookedData()?
	FConfigFile* PlatformConfigFile = GConfig->Find(GDeviceProfilesIni);
	const FString SectionName = FString::Printf(TEXT("%s %s"), *ProfileName, *UDeviceProfile::StaticClass()->GetName());
	return PlatformConfigFile->Contains(SectionName);
}

TArray<FString> UDeviceProfileManager::GetLoadableProfileNames(FName OptionalPlatformName) const
{
	FConfigCacheIni* ConfigSystem = GConfig;

	if (OptionalPlatformName != NAME_None)
	{
#if ALLOW_OTHER_PLATFORM_CONFIG
		ConfigSystem = FConfigCacheIni::ForPlatform(OptionalPlatformName);
#else

		checkf(OptionalPlatformName == FName(FPlatformProperties::IniPlatformName()), 
			TEXT("UDeviceProfileManager::GetLoadableProfileNames - This platform cannot load configurations for other platforms."));
#endif
	}

	TArray<FString> Results;
	FConfigFile* PlatformConfigFile = GConfig->Find(GDeviceProfilesIni);
	for (const TTuple<FString, FConfigSection>& Entry : AsConst(*PlatformConfigFile))
	{
		FString ProfileName;
		FString ProfileClass;

		if (Entry.Key.Split(" ", &ProfileName, &ProfileClass) && ProfileClass == *UDeviceProfile::StaticClass()->GetName())
		{
			Results.Add(ProfileName);
		}
	}

	return Results;
}


void UDeviceProfileManager::DeleteProfile( UDeviceProfile* Profile )
{
	Profiles.Remove( Profile );
}


UDeviceProfile* UDeviceProfileManager::FindProfile(const FString& ProfileName, bool bCreateProfileOnFail, FName OptionalPlatformName)
{
	UDeviceProfile* FoundProfile = nullptr;

	for( int32 Idx = 0; Idx < Profiles.Num(); Idx++ )
	{
		UDeviceProfile* CurrentDevice = CastChecked<UDeviceProfile>( Profiles[Idx] );
		if( CurrentDevice->GetFName() == *ProfileName )
		{
			FoundProfile = CurrentDevice;
			break;
		}
	}

	if ( bCreateProfileOnFail && FoundProfile == nullptr )
	{
		FString PlatformName = (OptionalPlatformName != NAME_None) ? OptionalPlatformName.ToString() : FString(FPlatformProperties::IniPlatformName());
		FoundProfile = CreateProfile(ProfileName, PlatformName);
	}
	return FoundProfile;
}


FOnDeviceProfileManagerUpdated& UDeviceProfileManager::OnManagerUpdated()
{
	return ManagerUpdatedDelegate;
}


FOnActiveDeviceProfileChanged& UDeviceProfileManager::OnActiveDeviceProfileChanged()
{
	return ActiveDeviceProfileChangedDelegate;
}

FWillReapplyDeviceProfiles& UDeviceProfileManager::WillReapplyDeviceProfiles()
{
	return PreReapplyDeviceProfilesDelegate;
}

FOnReapplyDeviceProfiles& UDeviceProfileManager::OnReapplyDeviceProfiles()
{
	return PostReapplyDeviceProfilesDelegate;
}

void UDeviceProfileManager::GetProfileConfigFiles(OUT TArray<FString>& OutConfigFiles)
{
	TSet<FString> SetOfPaths;

	// Make sure generic platform is first
	const FString RelativeConfigFilePath = FString::Printf(TEXT("%sDefault%ss.ini"), *FPaths::SourceConfigDir(), *UDeviceProfile::StaticClass()->GetName());
	SetOfPaths.Add(RelativeConfigFilePath);

	for (int32 DeviceProfileIndex = 0; DeviceProfileIndex < Profiles.Num(); ++DeviceProfileIndex)
	{
		UDeviceProfile* CurrentProfile = CastChecked<UDeviceProfile>(Profiles[DeviceProfileIndex]);
		SetOfPaths.Add(CurrentProfile->GetDefaultConfigFilename());
	}
	
	OutConfigFiles = SetOfPaths.Array();
}

void UDeviceProfileManager::LoadProfiles()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDeviceProfileManager::LoadProfiles);

	if( !HasAnyFlags( RF_ClassDefaultObject ) )
	{
		UE::ConfigAccessTracking::FIgnoreScope IgnoreConfigAccessTracking = UE::ConfigAccessTracking::FIgnoreScope();

		TMap<FString, FString> DeviceProfileToPlatformConfigMap;
		DeviceProfileToPlatformConfigMap.Add(TEXT("GlobalDefaults,None"), FPlatformProperties::IniPlatformName());
		
#if ALLOW_OTHER_PLATFORM_CONFIG
		TArray<FName> PlatformsToLoad = FDataDrivenPlatformInfoRegistry::GetSortedPlatformNames(EPlatformInfoType::TruePlatformsOnly);

		// go over all the platforms we find, starting with the current platform
		for (int32 PlatformIndex = 0; PlatformIndex <= PlatformsToLoad.Num(); PlatformIndex++)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UDeviceProfileManager::FindOrLoadPlatformConfig);

			// which platform's set of ini files should we load from?
			FString ConfigLoadPlatform = PlatformIndex == 0 ? FString(FPlatformProperties::IniPlatformName()) : PlatformsToLoad[PlatformIndex - 1].ToString();

			// load the DP.ini files (from current platform and then by the extra confidential platforms)
			FConfigFile LocalPlatformConfigFile;
			const FConfigFile* PlatformConfigFile = FConfigCacheIni::FindOrLoadPlatformConfig(LocalPlatformConfigFile, TEXT("DeviceProfiles"), *ConfigLoadPlatform);

			// load all of the DeviceProfiles
			TArray<FString> ProfileDescriptions;
			PlatformConfigFile->GetArray(TEXT("DeviceProfiles"), TEXT("DeviceProfileNameAndTypes"), ProfileDescriptions);

			// add them to our collection of profiles by platform
			for (const FString& Desc : ProfileDescriptions)
			{
				if (!DeviceProfileToPlatformConfigMap.Contains(Desc))
				{
					DeviceProfileToPlatformConfigMap.Add(Desc, ConfigLoadPlatform);
				}
			}
		}
#endif
		
		// now that we have gathered all the unique DPs, load them from the proper platform hierarchy
		for (auto It = DeviceProfileToPlatformConfigMap.CreateIterator(); It; ++It)
		{
			// the value of the map is in the format Name,DeviceType (DeviceType is usually platform)
			FString Name, DeviceType;
			It.Key().Split(TEXT(","), &Name, &DeviceType);

			if (FindObject<UDeviceProfile>(GetTransientPackage(), *Name) == NULL)
			{
				CreateProfile(Name, DeviceType, TEXT(""), *It.Value());
			}
		}

#if WITH_EDITOR
		if (!FPlatformProperties::RequiresCookedData())
		{
			// Register Texture LOD settings with each Target Platform
			ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
			const TArray<ITargetPlatform*>& TargetPlatforms = TargetPlatformManager.GetTargetPlatforms();
			for (int32 PlatformIndex = 0; PlatformIndex < TargetPlatforms.Num(); ++PlatformIndex)
			{
				ITargetPlatform* Platform = TargetPlatforms[PlatformIndex];

				// Set TextureLODSettings
				UDeviceProfile * DP = FindProfile(Platform->CookingDeviceProfileName(), false);
				checkf(DP, TEXT("No Device Profile found for %s"), *Platform->CookingDeviceProfileName());

#if ALLOW_OTHER_PLATFORM_CONFIG
				// disable redirect if we are using this for cooking.
				DP->SetPreviewTextureGroupRedirect(nullptr);
#endif

				const UTextureLODSettings* TextureLODSettingsObj = DP->GetTextureLODSettings();

				Platform->RegisterTextureLODSettings(TextureLODSettingsObj);
			}

			// Make backup copies to allow proper saving
			BackupProfiles.Reset();

			for (UDeviceProfile* DeviceProfile : Profiles)
			{
				FString DuplicateName = DeviceProfile->GetName() + BackupSuffix;
				UDeviceProfile* BackupProfile = DuplicateObject<UDeviceProfile>(DeviceProfile, DeviceProfile->GetOuter(), FName(*DuplicateName));
				BackupProfiles.Add(BackupProfile);
			}
		}
#endif

		ManagerUpdatedDelegate.Broadcast();
	}
}


void UDeviceProfileManager::SaveProfiles(bool bSaveToDefaults)
{
	if( !HasAnyFlags( RF_ClassDefaultObject ) )
	{
		if(bSaveToDefaults)
		{
			for (int32 DeviceProfileIndex = 0; DeviceProfileIndex < Profiles.Num(); ++DeviceProfileIndex)
			{
				UDeviceProfile* CurrentProfile = CastChecked<UDeviceProfile>(Profiles[DeviceProfileIndex]);
				FString BackupName = CurrentProfile->GetName() + BackupSuffix;
				UDeviceProfile* BackupProfile = FindObject<UDeviceProfile>(GetTransientPackage(), *BackupName);

				// Don't save if it hasn't changed
				if (!AreProfilesTheSame(CurrentProfile, BackupProfile))
				{
					// Strip out runtime inherited texture groups before save
					UDeviceProfile* ParentProfile = CurrentProfile->GetParentProfile(true);
					if (ParentProfile && CurrentProfile->TextureLODGroups.Num() == ParentProfile->TextureLODGroups.Num())
					{
						// Remove any that are the same, these are saved as a keyed array so the rest will inherit
						for (int32 i = CurrentProfile->TextureLODGroups.Num() - 1; i >= 0; i--)
						{
							if (CurrentProfile->TextureLODGroups[i] == ParentProfile->TextureLODGroups[i])
							{
								CurrentProfile->TextureLODGroups.RemoveAt(i);
							}
						}
					}
					
					CurrentProfile->TryUpdateDefaultConfigFile();

					// Recreate texture groups
					CurrentProfile->ValidateProfile();
				}
			}
		}
		else
		{
			// We do not want to save local changes to profiles as this is not how any other editor works and it confuses the user
			// For changes to save you need to hit the save to defaults button in the device profile editor
		}

		ManagerUpdatedDelegate.Broadcast();
	}
}

#if ALLOW_OTHER_PLATFORM_CONFIG
void UDeviceProfileManager::SetPreviewDeviceProfile(UDeviceProfile* DeviceProfile, FName PreviewModeTag)
{
	if (PreviewModeTag == NAME_None)
	{
		PreviewModeTag = "UnknownPreviewMode";
	}
	
	UE_LOGF(LogDeviceProfileManager, Log, "SetPreviewDeviceProfile Start : %ls %ls", *(DeviceProfile->GetName()), *(PreviewModeTag.ToString()) );

	RestorePreviewDeviceProfile(PreviewModeTag);

	PreviewDeviceProfile = DeviceProfile;
	
	ActiveDeviceProfile->SetPreviewTextureGroupRedirect(PreviewDeviceProfile);
	
#if WITH_EDITOR
	{
		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		ITargetPlatform* TargetPlatform = TPM->GetRunningTargetPlatform();
		
		UDeviceProfile * CDP = UDeviceProfileManager::Get().FindProfile(
			TargetPlatform->CookingDeviceProfileName());

		// usually Active == CDP
		// but if not then they must be kept in sync
		if ( CDP != ActiveDeviceProfile )
		{
			CDP->SetPreviewTextureGroupRedirect(PreviewDeviceProfile);
		}
	}
#endif

	// walk over all cvars and apply them
	IConsoleManager::Get().PreviewPlatformCVars(*DeviceProfile->DeviceType, DeviceProfile->GetName(), PreviewModeTag);

	// Gathering the cvars the device profile inherits and contains
	TMap<FName, FString> DeviceProfileCVars = GatherDeviceProfileCVars(DeviceProfile->GetName(), EDeviceProfileMode::DPM_CacheValues);
	for (TPair<FName, FString>& Pair : DeviceProfileCVars)
	{
		FString CVarKey = Pair.Key.ToString();
		const FString& CVarValue = Pair.Value;
		// Cache any scalability related cvars so that the active profile scalability group can be overriden
		const bool bIsScalabilityBucket = CVarKey.StartsWith(TEXT("sg."));
		if (bIsScalabilityBucket && CVarAllowScalabilityGroupsToChangeAtRuntime.GetValueOnGameThread() > 0)
		{
			PreviewDeviceProfileScalabilityCVars.Add(*CVarKey, *CVarValue);
		}
	}

	// broadcast cvar sinks now that we are done
	IConsoleManager::Get().CallAllConsoleVariableSinks();

	PreviewDeviceProfileChangedDelegate.Broadcast();
	
	UE_LOGF(LogDeviceProfileManager, Log, "SetPreviewDeviceProfile End : %ls %ls", *(DeviceProfile->GetName()), *(PreviewModeTag.ToString()) );

}


void UDeviceProfileManager::RestorePreviewDeviceProfile(FName PreviewModeTag)
{
	if (PreviewModeTag == NAME_None)
	{
		PreviewModeTag = "UnknownPreviewMode";
	}

	PreviewDeviceProfile = nullptr;

	ActiveDeviceProfile->SetPreviewTextureGroupRedirect(nullptr);
	
#if WITH_EDITOR
	{
		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		ITargetPlatform* TargetPlatform = TPM->GetRunningTargetPlatform();
		
		UDeviceProfile * CDP = UDeviceProfileManager::Get().FindProfile(
			TargetPlatform->CookingDeviceProfileName());

		// usually Active == CDP
		// but if not then they must be kept in sync
		if ( CDP != ActiveDeviceProfile )
		{
			CDP->SetPreviewTextureGroupRedirect(nullptr);
		}
	}
#endif

	PreviewDeviceProfileScalabilityCVars.Empty();

	IConsoleManager::Get().UnsetAllConsoleVariablesWithTag(PreviewModeTag, ECVF_SetByPreview);

	// broadcast cvar sinks now that we are done
	IConsoleManager::Get().CallAllConsoleVariableSinks();

	PreviewDeviceProfileChangedDelegate.Broadcast();
}
#endif

/**
* Overrides the device profile. The original profile can be restored with RestoreDefaultDeviceProfile
*/
void UDeviceProfileManager::SetOverrideDeviceProfile(UDeviceProfile* DeviceProfile)
{
	// If we're not already overriding record the BaseDeviceProfile
	if(!BaseDeviceProfile)
	{
		BaseDeviceProfile = DeviceProfileManagerSingleton->GetActiveProfile();
	}
	UE_LOGF(LogDeviceProfileManager, Log, "Overriding DeviceProfile to %ls, base device profile %ls", *DeviceProfile->GetName(), *BaseDeviceProfile->GetName() );

	// reset any fragments, this will cause them to be rematched.
	PlatformFragmentsSelected.Empty();
	// restore to the pre-DP cvar state:
	RestorePushedState(PushedSettings);

	// activate new one!
	DeviceProfileManagerSingleton->SetActiveDeviceProfile(DeviceProfile);
	InitializeCVarsForActiveDeviceProfile();

	// broadcast cvar sinks now that we are done
	IConsoleManager::Get().CallAllConsoleVariableSinks();
}

/**
* Restore the device profile to the default for this device
*/
void UDeviceProfileManager::RestoreDefaultDeviceProfile()
{
	// have we been overridden?
	if (BaseDeviceProfile)
	{
		UE_LOGF(LogDeviceProfileManager, Log, "Restoring overridden DP back to %ls", *BaseDeviceProfile->GetName());
		// this differs from previous behavior, we used to push only the cvar state that was modified by the override.
		// But now we restore the entire CVar state to 'pre-DP' stage and reapply the currently active DP.
		// reset the base profile as we are no longer overriding
		RestorePushedState(PushedSettings);
		// reset any fragments, this will cause them to be rematched.
		PlatformFragmentsSelected.Empty();
		SetActiveDeviceProfile(BaseDeviceProfile);
		BaseDeviceProfile = nullptr;
		//set the DP cvar state
		InitializeCVarsForActiveDeviceProfile();
	}
}

void UDeviceProfileManager::HandleDeviceProfileOverrideChange()
{
	FString CVarValue = CVarDeviceProfileOverride.GetValueOnGameThread();
	// only handle when the value is different
	if (CVarValue.Len() > 0 && CVarValue != GetActiveProfile()->GetName())
	{
		FString PlatformName = ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName());
		
		TArray<FString> DeviceProfileNameAndTypes;
		FConfigFile LocalConfigFile;
		const FConfigFile* PlatformConfigFile = FConfigCacheIni::FindOrLoadPlatformConfig(LocalConfigFile, TEXT("DeviceProfiles"), *PlatformName);
		PlatformConfigFile->GetArray(TEXT("DeviceProfiles"), TEXT("DeviceProfileNameAndTypes"), DeviceProfileNameAndTypes);
			
		bool bCreateIfMissing = false;
		for (const FString& Desc: DeviceProfileNameAndTypes)
		{
			FString Name, DeviceType;
			Desc.Split(TEXT(","), &Name, &DeviceType);
			if ((DeviceType == PlatformName) && (Name == CVarValue))
			{
				bCreateIfMissing = true;
				break;
			}

		}

		UDeviceProfile* NewActiveProfile = FindProfile(CVarValue, bCreateIfMissing);
		if (NewActiveProfile)
		{
			SetOverrideDeviceProfile(NewActiveProfile);
		}
	}
}

bool UDeviceProfileManager::AreProfilesTheSame(UDeviceProfile* Profile1, UDeviceProfile* Profile2) const
{
	if (!AreTextureGroupsTheSame(Profile1, Profile2))
	{
		// This does null check
		return false;
	}

	if (!Profile1->DeviceType.Equals(Profile2->DeviceType, ESearchCase::CaseSensitive))
	{
		return false;
	}

	if (!Profile1->BaseProfileName.Equals(Profile2->BaseProfileName, ESearchCase::CaseSensitive))
	{
		return false;
	}


	if (Profile1->CVars != Profile2->CVars)
	{
		return false;
	}

	if (Profile1->MatchingRules != Profile2->MatchingRules)
	{
		return false;
	}

	return true;
}

bool UDeviceProfileManager::AreTextureGroupsTheSame(UDeviceProfile* Profile1, UDeviceProfile* Profile2) const
{
	if (!Profile1 || !Profile2)
	{
		return false;
	}

	// If our groups are identical say yes
	if (Profile1->TextureLODGroups == Profile2->TextureLODGroups)
	{
		return true;
	}

	UDeviceProfile* Parent1 = Profile1->GetParentProfile(true);
	UDeviceProfile* Parent2 = Profile2->GetParentProfile(true);

	// Also if both profiles inherit groups with no changes, count them as the same
	if (Parent1 && Parent2 &&
		Profile1->TextureLODGroups == Parent1->TextureLODGroups &&
		Profile2->TextureLODGroups == Parent2->TextureLODGroups)
	{
		return true;
	}

	return false;
}

#if ALLOW_OTHER_PLATFORM_CONFIG && WITH_EDITOR
IDeviceProfileSelectorModule* UDeviceProfileManager::GetPreviewDeviceProfileSelectorModule(FConfigCacheIni* PreviewConfigSystemIn)
{
	// If we're getting the selector for previewing the PreviewDeviceProfileSelectionModule, this can be separately configured to preview a target device.
	FString PreviewDeviceProfileSelectionModuleName;
	if (PreviewConfigSystemIn->GetString(TEXT("DeviceProfileManager"), TEXT("PreviewDeviceProfileSelectionModule"), PreviewDeviceProfileSelectionModuleName, GEngineIni))
	{
		// this should only be specified when previewing.
		if (IDeviceProfileSelectorModule* DPSelectorModule = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(*PreviewDeviceProfileSelectionModuleName))
		{
			return DPSelectorModule;
		}
	}
	return nullptr;
}
#endif

IDeviceProfileSelectorModule* UDeviceProfileManager::GetDeviceProfileSelectorModule()
{
	static FString DeviceProfileSelectionModule;
	static bool bReadModuleNameFromConfig = GConfig->GetString(TEXT("DeviceProfileManager"), TEXT("DeviceProfileSelectionModule"), DeviceProfileSelectionModule, GEngineIni);

	if (bReadModuleNameFromConfig)
	{
		if (IDeviceProfileSelectorModule* DPSelectorModule = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(*DeviceProfileSelectionModule))
		{
			return DPSelectorModule;
		}
	}
	return nullptr;
}

const FString UDeviceProfileManager::GetPlatformDeviceProfileName()
{
	FString ActiveProfileName = FPlatformProperties::PlatformName();

	// look for a commandline override (never even calls into the selector plugin)
	FString OverrideProfileName;
	if (FParse::Value(FCommandLine::Get(), TEXT("DeviceProfile="), OverrideProfileName) || FParse::Value(FCommandLine::Get(), TEXT("DP="), OverrideProfileName))
	{
		return OverrideProfileName;
	}

	// look for cvar override
	OverrideProfileName = CVarDeviceProfileOverride.GetValueOnGameThread();
	if (OverrideProfileName.Len() > 0)
	{
		return OverrideProfileName;
	}

	if (IDeviceProfileSelectorModule* DPSelectorModule = GetDeviceProfileSelectorModule())
	{
		ActiveProfileName = DPSelectorModule->GetRuntimeDeviceProfileName();
	}

	return ActiveProfileName;
}


const FString UDeviceProfileManager::GetActiveDeviceProfileName()
{
	if(ActiveDeviceProfile != nullptr)
	{
		return ActiveDeviceProfile->GetName();
	}
	else
	{
		return GetPlatformDeviceProfileName();
	}
}

const FString UDeviceProfileManager::GetActiveProfileName()
{
	return GetPlatformDeviceProfileName();
}

bool UDeviceProfileManager::GetScalabilityCVar(const FString& CVarName, int32& OutValue)
{
	if (const FString* CVarValue = PreviewDeviceProfileScalabilityCVars.Find(CVarName))
	{
		TTypeFromString<int32>::FromString(OutValue, **CVarValue);
		return true;
	}
	if (const FString* CVarValue = DeviceProfileScalabilityCVars.Find(CVarName))
	{
		TTypeFromString<int32>::FromString(OutValue, **CVarValue);
		return true;
	}

	return false;
}

bool UDeviceProfileManager::GetScalabilityCVar(const FString& CVarName, float& OutValue)
{
	if (const FString* CVarValue = PreviewDeviceProfileScalabilityCVars.Find(CVarName))
	{
		TTypeFromString<float>::FromString(OutValue, **CVarValue);
		return true;
	}
	if (const FString* CVarValue = DeviceProfileScalabilityCVars.Find(CVarName))
	{
		TTypeFromString<float>::FromString(OutValue, **CVarValue);
		return true;
	}

	return false;
}

void UDeviceProfileManager::SetActiveDeviceProfile( UDeviceProfile* DeviceProfile )
{
	ActiveDeviceProfile = DeviceProfile;

	UE_LOGF(LogDeviceProfileManager, Verbose, "Available device profiles:");
	for (int32 Idx = 0; Idx < Profiles.Num(); ++Idx)
	{
		UDeviceProfile* Profile = Cast<UDeviceProfile>(Profiles[Idx]);
		const void* TextureLODGroupsAddr = Profile ? Profile->TextureLODGroups.GetData() : nullptr;
		const int32 NumTextureLODGroups = Profile ? Profile->TextureLODGroups.Num() : 0;
		UE_LOGF(LogDeviceProfileManager, Verbose, "\t[%p][%p %d] %ls, ", Profile, TextureLODGroupsAddr, NumTextureLODGroups, Profile ? *Profile->GetName() : TEXT("None"));
	}

	const void* TextureLODGroupsAddr = ActiveDeviceProfile ? ActiveDeviceProfile->TextureLODGroups.GetData() : nullptr;
	const int32 NumTextureLODGroups = ActiveDeviceProfile ? ActiveDeviceProfile->TextureLODGroups.Num() : 0;
	UE_LOGF(LogDeviceProfileManager, Log, "Active device profile: [%p][%p %d] %ls", ActiveDeviceProfile, TextureLODGroupsAddr, NumTextureLODGroups, ActiveDeviceProfile ? *ActiveDeviceProfile->GetName() : TEXT("None"));

	CSV_METADATA(TEXT("DeviceProfile"), *GetActiveDeviceProfileName());

	ActiveDeviceProfileChangedDelegate.Broadcast();

	// Update the crash context 
	FGenericCrashContext::SetEngineData(TEXT("DeviceProfile.Name"), GetActiveDeviceProfileName());
}


UDeviceProfile* UDeviceProfileManager::GetActiveProfile() const
{
	return ActiveDeviceProfile;
}

UDeviceProfile* UDeviceProfileManager::GetPreviewDeviceProfile() const
{
	return PreviewDeviceProfile;
}

void UDeviceProfileManager::GetAllPossibleParentProfiles(const UDeviceProfile* ChildProfile, OUT TArray<UDeviceProfile*>& PossibleParentProfiles) const
{
	for(auto& NextProfile : Profiles)
	{
		UDeviceProfile* ParentProfile = CastChecked<UDeviceProfile>(NextProfile);
		if (ParentProfile->DeviceType == ChildProfile->DeviceType && ParentProfile != ChildProfile)
		{
			bool bIsValidPossibleParent = true;

			UDeviceProfile* CurrentAncestor = ParentProfile;
			do
			{
				if(CurrentAncestor->BaseProfileName == ChildProfile->GetName())
				{
					bIsValidPossibleParent = false;
					break;
				}
				else
				{
					CurrentAncestor = CurrentAncestor->Parent != nullptr ? CastChecked<UDeviceProfile>(CurrentAncestor->Parent) : NULL;
				}
			} while(CurrentAncestor && bIsValidPossibleParent);

			if(bIsValidPossibleParent)
			{
				PossibleParentProfiles.Add(ParentProfile);
			}
		}
	}
}

const FString UDeviceProfileManager::GetActiveDeviceProfileMatchedFragmentsString(bool bEnabledOnly, bool bIncludeTags, bool bAlphaSort)
{
	return FragmentPropertyArrayToFragmentString(PlatformFragmentsSelected, bEnabledOnly, bIncludeTags, bAlphaSort);
}

#if ALLOW_OTHER_PLATFORM_CONFIG
static bool GetCVarForDeviceProfile( FOutputDevice& Ar, FString DPName, FString CVarName)
{
	UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(DPName, false);
	if (DeviceProfile == nullptr)
	{
		Ar.Logf(TEXT("Unable to find device profile %s"), *DPName);
		return false;
	}

	FString Value;
	const FString* DPValue = DeviceProfile->GetAllExpandedCVars().Find(CVarName);
	if (DPValue != nullptr)
	{
		Value = *DPValue;
	}
	else
	{
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
		if (!CVar)
		{
			Ar.Logf(TEXT("Unable to find cvar %s"), *CVarName);
			return false;
		}

		Value = CVar->GetDefaultValue();
	}

	Ar.Logf(TEXT("%s@%s = \"%s\""), *DPName, *CVarName, *Value);

	return true;
}

#endif

class FPlatformCVarExec : public FSelfRegisteringExec
{
protected:

	// FSelfRegisteringExec interface
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
#if ALLOW_OTHER_PLATFORM_CONFIG
		if (FParse::Command(&Cmd, TEXT("dpcvar")))
		{
			FString DPName, CVarName;
			if (FString(Cmd).Split(TEXT("@"), &DPName, &CVarName) == false)
			{
				return false;
			}

			return GetCVarForDeviceProfile(Ar, DPName, CVarName);
		}
		else if (FParse::Command(&Cmd, TEXT("dpdump")))
		{
			UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(Cmd, false);
			if (DeviceProfile)
			{
				Ar.Logf(TEXT("All cvars found for deviceprofile %s"), Cmd);
				for (const auto& Pair : DeviceProfile->GetAllExpandedCVars())
				{
					Ar.Logf(TEXT("%s = %s"), *Pair.Key, *Pair.Value);
				}

				// log out the LODGroups fully
				FArrayProperty* LODGroupsProperty = FindFProperty<FArrayProperty>(UDeviceProfile::StaticClass(), GET_MEMBER_NAME_CHECKED(UTextureLODSettings, TextureLODGroups));
				FScriptArrayHelper_InContainer ArrayHelper(LODGroupsProperty, DeviceProfile);
				for (int32 Index = 0; Index < ArrayHelper.Num(); Index++)
				{
					FString	Buffer;
					LODGroupsProperty->Inner->ExportTextItem_Direct(Buffer, ArrayHelper.GetRawPtr(Index), ArrayHelper.GetRawPtr(Index), DeviceProfile, 0);
					Ar.Logf(TEXT("LODGroup[%d]=%s"), Index, *Buffer);
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("dpdumppreview")))
		{
			UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(Cmd, false);
			if (DeviceProfile)
			{
				Ar.Logf(TEXT("All preview cvars found for deviceprofile %s"), Cmd);
				for (const auto& Pair : DeviceProfile->GetAllPreviewCVars())
				{
					Ar.Logf(TEXT("%s = %s"), *Pair.Key, *Pair.Value);
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("dppreview")))
		{
			UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(Cmd, false);
			if (DeviceProfile)
			{
				UDeviceProfileManager::Get().SetPreviewDeviceProfile(DeviceProfile);
			}
		}
		else if (FParse::Command(&Cmd, TEXT("dprestore")))
		{
			UDeviceProfileManager::Get().RestorePreviewDeviceProfile();
		}
		else if (FParse::Command(&Cmd, TEXT("dpreload")))
		{
			// clear cached other-platform CVars
			IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda(
				[](const TCHAR* Key, IConsoleObject* ConsoleObject)
				{
					if (IConsoleVariable* AsVariable = ConsoleObject->AsVariable())
					{
						AsVariable->ClearPlatformVariables();
					}

				}));

			// clear some cached cvars in other-platform expansions
			for (const TObjectPtr<UDeviceProfile>& DeviceProfile : UDeviceProfileManager::Get().Profiles)
			{
				DeviceProfile->ClearAllExpandedCVars();
			}

			FConfigCacheIni::ClearOtherPlatformConfigs();
		}
#else
		if (FParse::Command(&Cmd, TEXT("dpdump")))
		{
			UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(Cmd, false);
			if (DeviceProfile)
			{
				Ar.Logf(TEXT("All cvars found for deviceprofile %s"), Cmd);
				Ar.Logf(TEXT("  CVARS CURRENTLY DISABLED DUE TO ALLOW_OTHER_PLATFORM_CONFIG"), Cmd);

				// log out the LODGroups fully
				FArrayProperty* LODGroupsProperty = FindFProperty<FArrayProperty>(UDeviceProfile::StaticClass(), GET_MEMBER_NAME_CHECKED(UTextureLODSettings, TextureLODGroups));
				FScriptArrayHelper_InContainer ArrayHelper(LODGroupsProperty, DeviceProfile);
				for (int32 Index = 0; Index < ArrayHelper.Num(); Index++)
				{
					FString	Buffer;
					LODGroupsProperty->Inner->ExportTextItem_Direct(Buffer, ArrayHelper.GetRawPtr(Index), ArrayHelper.GetRawPtr(Index), DeviceProfile, 0);
					Ar.Logf(TEXT("LODGroup[%d]=%s"), Index, *Buffer);
				}
			}
		}
#endif
		else if (FParse::Command(&Cmd, TEXT("dpreapply")))
		{
			UDeviceProfileManager::Get().ReapplyDeviceProfile();
		}


		return false;
	}

} GPlatformCVarExec;

