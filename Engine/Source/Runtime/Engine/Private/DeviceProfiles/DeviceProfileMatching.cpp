// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceProfiles/DeviceProfileMatching.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Misc/CommandLine.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "IDeviceProfileSelectorModule.h"
#include "Internationalization/Regex.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Misc/SecureHash.h"
#include "UObject/PropertyPortFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeviceProfileMatching)

// Platform independent source types
static FName SRC_Chipset("SRC_Chipset");
static FName SRC_MakeAndModel("SRC_MakeAndModel");
static FName SRC_OSVersion("SRC_OSVersion");
static FName SRC_CommandLine("SRC_CommandLine");
static FName SRC_PrimaryGPUDesc("SRC_PrimaryGPUDesc");
static FName SRC_PreviousRegexMatch("SRC_PreviousRegexMatch");
static FName SRC_PreviousRegexMatch1("SRC_PreviousRegexMatch1");

// comparison operators:
static FName CMP_Equal("==");
static FName CMP_Less("<");
static FName CMP_LessEqual("<=");
static FName CMP_Greater(">");
static FName CMP_GreaterEqual(">=");
static FName CMP_NotEqual("!=");
static FName CMP_Regex("CMP_Regex");
static FName CMP_EqualIgnore("CMP_EqualIgnore");
static FName CMP_LessIgnore("CMP_LessIgnore");
static FName CMP_LessEqualIgnore("CMP_LessEqualIgnore");
static FName CMP_GreaterIgnore("CMP_GreaterIgnore");
static FName CMP_GreaterEqualIgnore("CMP_GreaterEqualIgnore");
static FName CMP_NotEqualIgnore("CMP_NotEqualIgnore");
static FName CMP_Hash("CMP_Hash");
static FName CMP_Or("OR");
static FName CMP_And("AND");

class FRulePropertiesContainer
{
	TMap<FName, FString> UserDefinedValues;
	bool RetrievePropertyValue(IDeviceProfileSelectorModule* DPSelectorModule, const FName& PropertyType, FString& PropertyValueOUT)
	{
		static const FString CommandLine = FCommandLine::Get();
		// universal properties
		if (PropertyType == SRC_Chipset) { PropertyValueOUT = FPlatformMisc::GetCPUChipset(); }
		else if (PropertyType == SRC_MakeAndModel) { PropertyValueOUT = FPlatformMisc::GetDeviceMakeAndModel(); }
		else if (PropertyType == SRC_OSVersion) { PropertyValueOUT = FPlatformMisc::GetOSVersion(); }
		else if (PropertyType == SRC_PrimaryGPUDesc) { PropertyValueOUT = FPlatformMisc::GetGPUDriverInfo(FPlatformMisc::GetPrimaryGPUBrand()).DeviceDescription; }
		else if (PropertyType == SRC_PreviousRegexMatch) { PropertyValueOUT = PreviousRegexMatches[0]; }
		else if (PropertyType == SRC_PreviousRegexMatch1) { PropertyValueOUT = PreviousRegexMatches[1]; }
		else if (PropertyType == SRC_CommandLine) { PropertyValueOUT = CommandLine; }
		// Selector module source data
		else if (DPSelectorModule && DPSelectorModule->GetSelectorPropertyValue(PropertyType, PropertyValueOUT)) { ; }
		// SetUserVar defined properties.
		else if (const FString* Value = UserDefinedValues.Find(PropertyType)) { PropertyValueOUT = *Value; }
		else
		{
			return false;
		}
		return true;
	}
public:
	FString PreviousRegexMatches[2];

	// ExpandVariables takes ParamString and expands any variable references $(...) into their values.
	// i.e. '$(SRC_GPUFamily)' becomes 'Adreno (TM) 640'
	// returns true if all variables were successfully expanded or if no variables were referenced. ExpandedResultOUT contains the result.
	// returns false if an error was encountered, ErrorsOUT will contain the error encountered. ExpandedResultOUT is untouched.
	// use '@' at the beginning of ParamString to treat the remainder of the string as a literal.
	bool ExpandVariables(IDeviceProfileSelectorModule* DPSelectorModule, const FString& ParamsString, FString& ExpandedResultOUT, FString& ErrorsOUT)
	{
		if (ParamsString.IsValidIndex(0) && ParamsString.GetCharArray()[0] == TEXT('@'))
		{
			ExpandedResultOUT = ParamsString.RightChop(1);
			return true;
		}

		ErrorsOUT.Reset();
		FString Result = ParamsString;

		int32 VarTokenIndex = INDEX_NONE;
		while ((VarTokenIndex = ParamsString.Find(TEXT("$("), ESearchCase::CaseSensitive, ESearchDir::FromEnd, VarTokenIndex)) != INDEX_NONE)
		{
			int32 VarTokenEndIndex = ParamsString.Find(TEXT(")"), ESearchCase::CaseSensitive, ESearchDir::FromStart, VarTokenIndex + 2);

			if (VarTokenEndIndex != INDEX_NONE)
			{
				FString SourceVarWithTokens = ParamsString.Mid(VarTokenIndex, (VarTokenEndIndex - VarTokenIndex) + 1);
				FString SourceVar = SourceVarWithTokens.Mid(2, (VarTokenEndIndex - VarTokenIndex) - 2);
				int32 FindIdxDummy;
				FString RetrievedValue;
				if (SourceVar.FindChar(TEXT('$'), FindIdxDummy) || SourceVar.FindChar(TEXT('('), FindIdxDummy) || !RetrievePropertyValue(DPSelectorModule, FName(SourceVar), RetrievedValue))
				{
					ErrorsOUT = FString::Printf(TEXT("could not find %s (from %s)"), *SourceVarWithTokens, *ParamsString);
					return false;
				}
				else
				{
					Result.RemoveAt(VarTokenIndex, (VarTokenEndIndex - VarTokenIndex)+1);
					Result.InsertAt(VarTokenIndex, *RetrievedValue);
				}
			}
			else
			{
				ErrorsOUT = FString::Printf(TEXT("no closing parenthesis for %s"), *ParamsString);
				return false;
			}
		}
		ExpandedResultOUT = Result;
		return true;
	}

	void SetUserDefinedValue(FName PropertyName, const FString& PropertyValue)
	{
		UserDefinedValues.Add(PropertyName, PropertyValue);
	}
};

class FRuleMatchRunner
{
	FRulePropertiesContainer& RuleProperties;
	FOutputDevice* ErrorDevice;

public:
	struct FDeviceProfileMatchCriterion
	{
		FString Arg1;
		FName Operator;
		FString Arg2;
	};

	FRuleMatchRunner(FRulePropertiesContainer& RulePropertiesIN, FOutputDevice* ErrorDeviceIn) : RuleProperties(RulePropertiesIN), ErrorDevice(ErrorDeviceIn){}
	bool ProcessRule(IDeviceProfileSelectorModule* DPSelectorModule, const FDeviceProfileMatchCriterion& DeviceProfileMatchCriterion, const FString& RuleName)
	{
		bool bCurrentMatch = true;

		const FString& SourceString = DeviceProfileMatchCriterion.Arg1;
		FName CompareType = DeviceProfileMatchCriterion.Operator;
		const FString& MatchString = DeviceProfileMatchCriterion.Arg2;

		const bool bNumericOperands = SourceString.IsNumeric() && MatchString.IsNumeric();

		if (CompareType == CMP_Equal)
		{
			if (SourceString != MatchString)
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_Less)
		{
			if ((bNumericOperands && FCString::Atof(*SourceString) >= FCString::Atof(*MatchString)) || (!bNumericOperands && SourceString >= MatchString))
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_LessEqual)
		{
			if ((bNumericOperands && FCString::Atof(*SourceString) > FCString::Atof(*MatchString)) || (!bNumericOperands && SourceString > MatchString))
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_Greater)
		{
			if ((bNumericOperands && FCString::Atof(*SourceString) <= FCString::Atof(*MatchString)) || (!bNumericOperands && SourceString <= MatchString))
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_GreaterEqual)
		{
			if ((bNumericOperands && FCString::Atof(*SourceString) < FCString::Atof(*MatchString)) || (!bNumericOperands && SourceString < MatchString))
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_NotEqual)
		{
			if (*SourceString == MatchString)
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_Or || CompareType == CMP_And)
		{
			bool bArg1, bArg2;

			if (bNumericOperands)
			{
				bArg1 = FCString::Atoi(*SourceString) != 0;
				bArg2 = FCString::Atoi(*MatchString) != 0;
			}
			else
			{
				static const FString TrueString("true");
				bArg1 = SourceString == TrueString;
				bArg2 = MatchString == TrueString;
			}

			if (CompareType == CMP_Or)
			{
				bCurrentMatch = (bArg1 || bArg2);
			}
			else
			{
				bCurrentMatch = (bArg1 && bArg2);
			}
		}
		else if (CompareType == CMP_EqualIgnore)
		{
			if (SourceString.ToLower() != MatchString.ToLower())
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_LessIgnore)
		{
			if (SourceString.ToLower() >= MatchString.ToLower())
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_LessEqualIgnore)
		{
			if (SourceString.ToLower() > MatchString.ToLower())
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_GreaterIgnore)
		{
			if (SourceString.ToLower() <= MatchString.ToLower())
			{
				bCurrentMatch = false;
			}
			else if (CompareType == CMP_GreaterEqualIgnore)
			{
				if (SourceString.ToLower() < MatchString.ToLower())
				{
					bCurrentMatch = false;
				}
			}
		}
		else if (CompareType == CMP_NotEqualIgnore)
		{
			if (SourceString.ToLower() == MatchString.ToLower())
			{
				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_Regex)
		{
			const FRegexPattern RegexPattern(MatchString);
			FRegexMatcher RegexMatcher(RegexPattern, SourceString);
			if (RegexMatcher.FindNext())
			{
				RuleProperties.PreviousRegexMatches[0] = RegexMatcher.GetCaptureGroup(1);
				RuleProperties.PreviousRegexMatches[1] = RegexMatcher.GetCaptureGroup(2);
			}
			else
			{
				RuleProperties.PreviousRegexMatches[0].Empty();
				RuleProperties.PreviousRegexMatches[1].Empty();

				bCurrentMatch = false;
			}
		}
		else if (CompareType == CMP_Hash)
		{
			// Salt string is concatenated onto the end of the input text.
			// For example the input string "PhoneModel" with salt "Salt" and pepper "Pepper" can be computed with
			// % printf "PhoneModelSaltPepper" | openssl dgst -sha1 -hex
			// resulting in d9e5cbd6b0e4dba00edd9de92cf64ee4c3f3a2db and would be stored in the matching rules as 
			// "Salt|d9e5cbd6b0e4dba00edd9de92cf64ee4c3f3a2db". Salt is optional.
			FString MatchHashString;
			FString SaltString;
			if (!MatchString.Split(TEXT("|"), &SaltString, &MatchHashString))
			{
				MatchHashString = MatchString;
			}
			FString HashInputString = SourceString + SaltString
#ifdef HASH_PEPPER_SECRET_GUID
				+ HASH_PEPPER_SECRET_GUID.ToString()
#endif
				;

			FSHAHash SourceHash;
			FSHA1::HashBuffer(TCHAR_TO_ANSI(*HashInputString), HashInputString.Len(), SourceHash.Hash);
			if (SourceHash.ToString() != MatchHashString.ToUpper())
			{
				bCurrentMatch = false;
			}
		}
		else
		{
			ErrorDevice->Logf(TEXT("rule %s unknown operator type, %s"), *RuleName, *CompareType.ToString());
	
			bCurrentMatch = false;
		}

		return bCurrentMatch;
	}
};

static bool EvaluateMatch(IDeviceProfileSelectorModule* DPSelectorModule, const FString& RuleName, const FDPMatchingRulestructBase& rule, FRulePropertiesContainer& RuleProperties, FString& SelectedFragmentsOUT, FOutputDevice* Errors)
{
	if (!rule.AppendFragments.IsEmpty())
	{
		if (!SelectedFragmentsOUT.IsEmpty())
		{
			SelectedFragmentsOUT += TEXT(",");
		}

		SelectedFragmentsOUT += rule.AppendFragments;
	}

	if (!rule.SetUserVar.IsEmpty())
	{
		TArray<FString> NewSRCs;
		rule.SetUserVar.ParseIntoArray(NewSRCs, TEXT(","), true);
		for (FString& SRCentry : NewSRCs)
		{
			FString CSRCType, CSRCValue;
			if (SRCentry.Split(TEXT("="), &CSRCType, &CSRCValue))
			{
				FString ExpandedValue;
				FString ErrorStr;
				if (!RuleProperties.ExpandVariables(DPSelectorModule, CSRCValue, ExpandedValue, ErrorStr))
				{
					Errors->Logf(TEXT("MatchesRules: rule %s failed. SetSRC %s= could not find %s"), *RuleName, *CSRCType, *ErrorStr);
				}
				else
				{
					RuleProperties.SetUserDefinedValue(FName(CSRCType), ExpandedValue);
					UE_LOGF(LogInit, Verbose, "MatchesRules: Adding source %ls : %ls", *CSRCType, *ExpandedValue);
				}
			}
		}
	}

	TArray<FDPMatchingIfCondition> Expression = rule.IfConditions;
	if (Expression.Num() == 0)
	{
		UE_LOGF(LogInit, Verbose, "MatchesRules: %ls = true (no rules specified)", *RuleName);
		return true;
	}

	// if no operator specified, insert an implicit AND operator.
	for (int i = 0; i < Expression.Num(); i++)
	{
		// if this and next are )( operations insert AND
		if (i + 1 < Expression.Num() && (!Expression[i].Arg1.IsEmpty() || Expression[i].Operator == TEXT(")")))
		{
			if (!Expression[i + 1].Arg1.IsEmpty() || Expression[i + 1].Operator == TEXT("("))
			{
				FDPMatchingIfCondition ImplicitAnd;
				ImplicitAnd.Operator = TEXT("AND");
				Expression.Insert(ImplicitAnd, i + 1);
				i++;
			}
		}
	}

	FString Line;
	for (int i = 0; i < Expression.Num(); i++)
	{
		if (!(Expression[i].Arg1.IsEmpty()))
		{
			Line += FString::Printf(TEXT("(%s %s %s)"), *Expression[i].Arg1, *Expression[i].Operator.ToString(), *Expression[i].Arg2);
		}
		else
		{
			Line += FString::Printf(TEXT(" %s "), *Expression[i].Operator.ToString());
		}
	}
	UE_LOGF(LogInit, Verbose, "MatchesRules: rule %ls : %ls", *RuleName, *Line);

	struct FExpressionItem
	{
		FExpressionItem(FString InValue, bool bInOperator) : Value(InValue), bOperator(bInOperator) {}
		FString Value;
		bool bOperator;
	};

	TArray<FExpressionItem> RPNOutput;
	TArray<FString> Operators;

	auto PushOperand = [&RPNOutput](FString InValue) { RPNOutput.Push(FExpressionItem(InValue, false)); };
	auto PushOperator = [&RPNOutput](FString InValue)
	{
		RPNOutput.Push(FExpressionItem(InValue, true));
	};

	for (const FDPMatchingIfCondition& element : Expression)
	{
		if (!element.Arg1.IsEmpty())
		{
			PushOperand(element.Arg1);
			PushOperand(element.Arg2);

			if (!element.Operator.IsNone())
			{
				PushOperator(element.Operator.ToString());
			}
		}
		else if (!element.Operator.IsNone())
		{
			if (element.Operator == FName(")"))
			{
				while (Operators.Num())
				{
					FString PoppedOperator = Operators.Pop();
					if (PoppedOperator != FString("("))
					{
						PushOperator(PoppedOperator);
					}
					else
					{
						break;
					}
				}
			}
			else
			{
				Operators.Push(element.Operator.ToString());
			}
		}
	}
	while (Operators.Num())
	{
		FString PoppedOperator = Operators.Pop();
		if (PoppedOperator == FString("(") || PoppedOperator == FString(")"))
		{
			Errors->Logf(TEXT("MatchesRules: rule %s failed due to mismatching parenthesis! %s"), *RuleName, *PoppedOperator);
			return false;
		}
		PushOperator(PoppedOperator);
	}

	FRuleMatchRunner MatchMe(RuleProperties, Errors);
	TArray<FExpressionItem> ExpressionStack;
	for (int i = 0; i < RPNOutput.Num(); i++)
	{
		if (!RPNOutput[i].bOperator)
		{
			ExpressionStack.Push(RPNOutput[i]);
		}
		else
		{
			FExpressionItem B = ExpressionStack.Pop();
			FExpressionItem A = ExpressionStack.Pop();
			FRuleMatchRunner::FDeviceProfileMatchCriterion crit;
			crit.Operator = FName(RPNOutput[i].Value);

			// process the arguments and dereference any $(variables) to their content.
			FString ErrorStr;
			if (!RuleProperties.ExpandVariables(DPSelectorModule, A.Value, crit.Arg1, ErrorStr))
			{
				Errors->Logf(TEXT("rule %s failed to expand Arg1, %s"), *RuleName, *ErrorStr);
				crit.Arg1 = TEXT("[UnknownArg1Variable]");
			}
			if (!RuleProperties.ExpandVariables(DPSelectorModule, B.Value, crit.Arg2, ErrorStr))
			{
				Errors->Logf(TEXT("rule %s failed to expand Arg2, %s"), *RuleName, *ErrorStr);
				crit.Arg2 = TEXT("[UnknownArg2Variable]");
			}

			bool bResult = MatchMe.ProcessRule(DPSelectorModule, crit, RuleName);
			FExpressionItem C(bResult ? TEXT("true") : TEXT("false"), false);
			ExpressionStack.Push(C);

			UE_LOGF(LogInit, Verbose, "MatchesRules: rule %ls evaluating {%ls %ls %ls} = {%ls %ls %ls} = %ls", *RuleName, *A.Value, *RPNOutput[i].Value, *B.Value, *crit.Arg1, *RPNOutput[i].Value, *crit.Arg2, *C.Value);
		}
	}
	FExpressionItem Result = ExpressionStack.Pop();
	UE_LOGF(LogInit, Log, "MatchesRules: rule %ls = %ls", *RuleName, *Result.Value);

	const bool bMatched = Result.Value == TEXT("true");
	const FDPMatchingRulestructBase* Next = bMatched ? rule.GetOnTrue() : rule.GetOnFalse();

	bool bSuccess = true;
	if (Next)
	{
		// Generate a name from the parent and the new rule to help trace problems.
		static const FString NoName("<unnamed>");
		const FString NewRuleName = RuleName + TEXT("::")+(Next->RuleName.IsEmpty() ? NoName : Next->RuleName);

		bSuccess = EvaluateMatch(DPSelectorModule, NewRuleName, *Next, RuleProperties, SelectedFragmentsOUT, Errors);
	}

	return bSuccess;
}

// Convert a string of fragment names to a FSelectedFragmentProperties array.
// FragmentName1,FragmentName2,[optionaltag]FragmentName3, etc.
static TArray<FSelectedFragmentProperties> FragmentStringToFragmentProperties(const FString& FragmentString)
{
	TArray<FSelectedFragmentProperties> FragmentPropertiesList;
	TArray<FString> AppendedFragments;
	FragmentString.ParseIntoArray(AppendedFragments, TEXT(","), true);
	for (const FString& Fragment : AppendedFragments)
	{
		FSelectedFragmentProperties FragmentProperties;
		int32 TagDeclStart = Fragment.Find(TEXT("["));
		if (TagDeclStart >= 0)
		{
			int32 TagDeclEnd = Fragment.Find(TEXT("]"));
			if (TagDeclEnd > TagDeclStart)
			{
				FName Tag(Fragment.Mid(TagDeclStart + 1, (TagDeclEnd - TagDeclStart)-1));
				FragmentProperties.Tag = Tag;
				FragmentProperties.bEnabled = false;
				FragmentProperties.Fragment = Fragment.RightChop(TagDeclEnd + 1);
				FragmentPropertiesList.Emplace(MoveTemp(FragmentProperties));
			}
		}
		else
		{
			FragmentProperties.Fragment = Fragment;
			FragmentProperties.bEnabled = true;
			FragmentPropertiesList.Emplace(MoveTemp(FragmentProperties));
		}
	}
	return FragmentPropertiesList;
}

const FString UDeviceProfileManager::FragmentPropertyArrayToFragmentString(const TArray<FSelectedFragmentProperties>& FragmentProperties, bool bEnabledOnly, bool bIncludeTags,bool bAlphaSort)
{
	FString MatchedFragmentString;
	if(!FragmentProperties.IsEmpty())
	{
		TArray<FString> FragmentNames;
		for (const FSelectedFragmentProperties& FragmentProperty : FragmentProperties)
		{
			if (bEnabledOnly == false || FragmentProperty.bEnabled)
			{
				FString FragmentName;
				if (bIncludeTags && FragmentProperty.Tag != NAME_None)
				{
					FragmentName = TEXT("[") + FragmentProperty.Tag.ToString() + TEXT("]");
				}
				FragmentNames.Add(FragmentProperty.Fragment);
			}
		}
		if (bAlphaSort)
		{
			FragmentNames.Sort();
		}

		for (FString& Name : FragmentNames)
		{
			MatchedFragmentString += MatchedFragmentString.IsEmpty() ? Name : TEXT(",") + Name;
		}
	}
	return MatchedFragmentString;
}

class FDeviceProfileMatchingErrorContext: public FOutputDevice
{
public:
	FString Stage;
	int32 NumErrors;

	FDeviceProfileMatchingErrorContext()
		: FOutputDevice()
		, NumErrors(0)
	{}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		UE_LOGF(LogInit, Error, "DeviceProfileMatching: Error while parsing Matching Rules (%ls) : %ls", *Stage, V);
		NumErrors++;
	}
};


static TArray<FString> GetMatchedRulesArray(const FString& ParentDP, FConfigCacheIni* ConfigSystem)
{
	FString SectionSuffix = *FString::Printf(TEXT(" %s"), *UDeviceProfile::StaticClass()->GetName());
	FString CurrentSectionName = ParentDP + SectionSuffix;
	FString ArrayName("MatchingRules");
	TArray<FString> MatchingRulesArray;
	ConfigSystem->GetArray(*CurrentSectionName, *ArrayName, MatchingRulesArray, GDeviceProfilesIni);

	return MatchingRulesArray;
}

static bool DPHasMatchingRules(const FString& ParentDP, FConfigCacheIni* ConfigSystem)
{
	return GetMatchedRulesArray(ParentDP, ConfigSystem).Num() > 0;
}

static FDPMatchingRulestruct ParseToRuleStruct(const FString& RuleText, FDeviceProfileMatchingErrorContext& DPMatchingErrorOutput)
{
	FDPMatchingRulestruct RuleStruct;
	FDPMatchingRulestruct::StaticStruct()->ImportText(*RuleText, &RuleStruct, nullptr, EPropertyPortFlags::PPF_None, &DPMatchingErrorOutput, FDPMatchingRulestruct::StaticStruct()->GetName(), true);
	return RuleStruct;
}

static FString LoadAndProcessMatchingRulesFromConfig(const FString& ParentDP, IDeviceProfileSelectorModule* DPSelector, FConfigCacheIni* ConfigSystem)
{
	const TArray<FString> MatchingRulesArray = GetMatchedRulesArray(ParentDP, ConfigSystem);

	FRulePropertiesContainer RuleProperties;
	FString SelectedFragments;
	FDeviceProfileMatchingErrorContext DPMatchingErrorOutput;
	int Count = 0;
	for(const FString& RuleText : MatchingRulesArray)
	{
		DPMatchingErrorOutput.Stage = FString::Printf(TEXT("%s rule #%d"), *ParentDP, Count);
		Count++;
		FDPMatchingRulestruct RuleStruct = ParseToRuleStruct(RuleText, DPMatchingErrorOutput);
		static const FString NoName("<unnamed>");
		const FString& RuleName = RuleStruct.RuleName.IsEmpty() ? NoName : RuleStruct.RuleName;
		EvaluateMatch(DPSelector, RuleName, RuleStruct, RuleProperties, SelectedFragments, &DPMatchingErrorOutput);
	}
	if (IsRunningCookCommandlet())
	{
		UE_CLOGF(DPMatchingErrorOutput.NumErrors > 0, LogInit, Warning, "DeviceProfileMatching: %d Warnings(s) encountered while processing MatchedRules for %ls", DPMatchingErrorOutput.NumErrors, *ParentDP);
	}
	else
	{
		UE_CLOGF(DPMatchingErrorOutput.NumErrors > 0, LogInit, Error, "DeviceProfileMatching: %d Error(s) encountered while processing MatchedRules for %ls", DPMatchingErrorOutput.NumErrors, *ParentDP);
	}

	FGenericCrashContext::SetEngineData(TEXT("MatchingDPStatus"), ParentDP + (DPMatchingErrorOutput.NumErrors > 0 ? TEXT("Error") : TEXT("No errors")));

	return SelectedFragments;
}

static FString RemoveAllWhiteSpace(const FString& StringIN)
{
	FString Ret;
	Ret.Reserve(StringIN.Len());
	for (TCHAR Character : StringIN)
	{
		if (!FChar::IsWhitespace(Character))
		{
			Ret += Character;
		}
	}
	return Ret;
}

 static void FindAllReferencedMatchedFragments(FString& AllAppendableFragments, const FDPMatchingRulestructBase* RuleStruct, FDeviceProfileMatchingErrorContext& DPMatchingErrorOutput)
 {
	if (!RuleStruct)
		return;

	FindAllReferencedMatchedFragments(AllAppendableFragments, RuleStruct->GetOnTrue(), DPMatchingErrorOutput);
	FindAllReferencedMatchedFragments(AllAppendableFragments, RuleStruct->GetOnFalse(), DPMatchingErrorOutput);

	if (!AllAppendableFragments.IsEmpty() && !RuleStruct->AppendFragments.IsEmpty())
	{
		AllAppendableFragments += TEXT(",");
	}
	AllAppendableFragments += RuleStruct->AppendFragments;
 }

TArray<FString> UDeviceProfileManager::FindAllReferencedFragmentsFromMatchedRules(const FString& ParentDP, FConfigCacheIni* ConfigSystem)
{
	TArray<FString> MatchingRulesArray = GetMatchedRulesArray(ParentDP, ConfigSystem);

	FDeviceProfileMatchingErrorContext DPMatchingErrorOutput;
	int Count = 0;
	FString AllAppendableFragments;
	for (const FString& RuleText : MatchingRulesArray)
	{
		DPMatchingErrorOutput.Stage = FString::Printf(TEXT("%s rule #%d"), *ParentDP, Count++);
		FDPMatchingRulestruct RuleStruct = ParseToRuleStruct(RuleText, DPMatchingErrorOutput);
		static const FString NoName("<unnamed>");
		FindAllReferencedMatchedFragments(AllAppendableFragments, &RuleStruct, DPMatchingErrorOutput);
	}
	UE_CLOGF(DPMatchingErrorOutput.NumErrors > 0, LogInit, Error, "DeviceProfileMatching: %d Error(s) encountered while processing FindAllReferencedFragments for %ls", DPMatchingErrorOutput.NumErrors, *ParentDP);

	AllAppendableFragments = RemoveAllWhiteSpace(AllAppendableFragments);
	TArray<FSelectedFragmentProperties> AllFragmentsSeen = FragmentStringToFragmentProperties(AllAppendableFragments);
	TArray<FString> UniqueMatchedFragments;

	// Return a unique list of the referenced fragments (including tagged/disabled)
	for (const FSelectedFragmentProperties& FragmentProperties : AllFragmentsSeen)
	{ 
		UniqueMatchedFragments.AddUnique(FragmentProperties.Fragment);
	}

	return UniqueMatchedFragments;
}

TArray<FSelectedFragmentProperties> UDeviceProfileManager::FindMatchingFragments(const FString& ParentDP, FConfigCacheIni* ConfigSystem)
{
	FString SelectedFragments;
#if !UE_BUILD_SHIPPING
	// Override selected fragments with commandline specified list -DPFragments=fragmentname,fragmentname2,[taggedname]fragment,... 
	FString DPFragmentString;
	if (FParse::Value(FCommandLine::Get(), TEXT("DPFragments="), DPFragmentString, false))
	{
		SelectedFragments = DPFragmentString;
	}
	else
#endif
	{
		IDeviceProfileSelectorModule* PreviewDPSelector = nullptr;
#if ALLOW_OTHER_PLATFORM_CONFIG && WITH_EDITOR
		if (ConfigSystem != GConfig)
		{
			PreviewDPSelector = GetPreviewDeviceProfileSelectorModule(ConfigSystem);
			// previewing a DP with matching rules will run if conditions with the host device's data sources. It will likely not match the preview device's behavior.
			UE_CLOGF(!PreviewDPSelector && DPHasMatchingRules(ParentDP, ConfigSystem), LogInit, Warning, "Preview DP %ls contains fragment matching rules, but no preview profile selector was found. The selected fragments for %ls will likely not match the behavior of the intended preview device.", *ParentDP, *ParentDP);
		}
#endif
		IDeviceProfileSelectorModule* DPSelector = PreviewDPSelector ? PreviewDPSelector : GetDeviceProfileSelectorModule();
		SelectedFragments = LoadAndProcessMatchingRulesFromConfig(ParentDP, DPSelector, ConfigSystem);
		
	}
	SelectedFragments = RemoveAllWhiteSpace(SelectedFragments);

	UE_CLOGF(!SelectedFragments.IsEmpty(), LogInit, Log, "MatchesRules:Fragment string %ls", *SelectedFragments);
	TArray<FSelectedFragmentProperties> MatchedFragments = FragmentStringToFragmentProperties(SelectedFragments);

	UE_CLOGF(MatchedFragments.Num()>0, LogInit, Log, "MatchesRules: MatchedFragments:");
	for (FSelectedFragmentProperties& MatchedFrag : MatchedFragments)
	{
		UE_CLOGF(MatchedFrag.Tag == NAME_None, LogInit, Log, "MatchesRules: %ls, enabled %d", *MatchedFrag.Fragment, MatchedFrag.bEnabled);
		UE_CLOGF(MatchedFrag.Tag != NAME_None, LogInit, Log, "MatchesRules: %ls=%ls, enabled %d", *MatchedFrag.Tag.ToString(), *MatchedFrag.Fragment, MatchedFrag.bEnabled);
	}

	return MatchedFragments;
}

