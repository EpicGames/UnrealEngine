// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/DialogueTypes.h"
#include "Sound/DialogueVoice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DialogueTypes)

bool operator==(const FDialogueContext& LHS, const FDialogueContext& RHS)
{
	return	LHS.Speaker == RHS.Speaker &&
		LHS.Targets == RHS.Targets;
}

bool operator!=(const FDialogueContext& LHS, const FDialogueContext& RHS)
{
	return	LHS.Speaker != RHS.Speaker ||
		LHS.Targets != RHS.Targets;
}

FDialogueContext::FDialogueContext()
	: Speaker(nullptr)
{
	Targets.AddZeroed();
}

FString FDialogueContext::GetContextHash() const
{
	uint32 SpeakerHash = 0;
	if (Speaker)
	{
		SpeakerHash = FCrc::MemCrc32(&Speaker->LocalizationGUID, sizeof(FGuid), SpeakerHash);
	}

	TArray<FGuid> UniqueSortedTargetGUIDs;
	for (const UDialogueVoice* Target : Targets)
	{
		if (Target)
		{
			UniqueSortedTargetGUIDs.AddUnique(Target->LocalizationGUID);
		}
	}
	UniqueSortedTargetGUIDs.Sort([](const FGuid& One, const FGuid& Two) -> bool
	{
		return One.ToString() < Two.ToString();
	});

	uint32 TargetHash = 0;
	for (const FGuid& TargetGUID : UniqueSortedTargetGUIDs)
	{
		TargetHash = FCrc::MemCrc32(&TargetGUID, sizeof(FGuid), TargetHash);
	}

	return FString::Printf(TEXT("%08X%08X"), SpeakerHash, TargetHash);
}

FDialogueWaveParameter::FDialogueWaveParameter()
	: DialogueWave(nullptr)
{

}

