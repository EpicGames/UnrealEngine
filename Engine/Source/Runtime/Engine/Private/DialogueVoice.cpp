// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/DialogueVoice.h"
#include "Sound/DialogueTypes.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DialogueVoice)

UDialogueVoice::UDialogueVoice(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LocalizationGUID( IsTemplate() ? FGuid() : FGuid::NewGuid() )
{
}

// Begin UObject interface. 
bool UDialogueVoice::IsReadyForFinishDestroy()
{
	return true;
}

FName UDialogueVoice::GetExporterName()
{
	return NAME_None;
}

FString UDialogueVoice::GetDesc()
{
	FString SummaryString;
	{
		FByteProperty* GenderProperty = CastFieldChecked<FByteProperty>(FindFieldChecked<FProperty>(GetClass(), GET_MEMBER_NAME_CHECKED(UDialogueVoice, Gender)));
		SummaryString += GenderProperty->Enum->GetDisplayNameTextByValue(Gender).ToString();

		if( Plurality != EGrammaticalNumber::Singular )
		{
			FByteProperty* PluralityProperty = CastFieldChecked<FByteProperty>( FindFieldChecked<FProperty>( GetClass(), GET_MEMBER_NAME_CHECKED(UDialogueVoice, Plurality)) );

			SummaryString += ", ";
			SummaryString += PluralityProperty->Enum->GetDisplayNameTextByValue(Plurality).ToString();
		}
	}

	return FString::Printf( TEXT( "%s (%s)" ), *( GetName() ), *(SummaryString) );
}

void UDialogueVoice::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UDialogueVoice::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
}

void UDialogueVoice::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if( !bDuplicateForPIE )
	{
		LocalizationGUID = FGuid::NewGuid();
	}
}
// End UObject interface. 

