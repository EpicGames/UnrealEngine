// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/DialogueWave.h"

#include "ActiveSound.h"
#include "Internationalization/GatherableTextData.h"
#include "Misc/StringFormatArg.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "Sound/DialogueSoundWaveProxy.h"
#include "Sound/DialogueVoice.h"
#include "Sound/SoundWave.h"
#include "SubtitleManager.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DialogueWave)

const FString FDialogueConstants::DialogueNamespace						= TEXT("Dialogue");
const FString FDialogueConstants::DialogueNotesNamespace				= TEXT("DialogueNotes");
const FString FDialogueConstants::SubtitleKeySuffix						= TEXT("_Subtitle");
#if WITH_EDITORONLY_DATA
const FString FDialogueConstants::ActingDirectionKeySuffix				= TEXT("_ActingDirection");
const FString FDialogueConstants::PropertyName_AudioFile				= TEXT("AudioFile");
const FString FDialogueConstants::PropertyName_VoiceActorDirection		= TEXT("VoiceActorDirection");
const FString FDialogueConstants::PropertyName_Speaker					= TEXT("Speaker");
const FString FDialogueConstants::PropertyName_Targets					= TEXT("Targets");
const FString FDialogueConstants::PropertyName_GrammaticalGender		= TEXT("Gender");
const FString FDialogueConstants::PropertyName_GrammaticalPlurality		= TEXT("Plurality");
const FString FDialogueConstants::PropertyName_TargetGrammaticalGender	= TEXT("TargetGender");
const FString FDialogueConstants::PropertyName_TargetGrammaticalNumber	= TEXT("TargetPlurality");
const FString FDialogueConstants::PropertyName_DialogueContext			= TEXT("Context");
const FString FDialogueConstants::PropertyName_IsMature					= FString(FLocMetadataObject::COMPARISON_MODIFIER_PREFIX) + TEXT("IsMature");
#endif //WITH_EDITORONLY_DATA

bool bUseNewSubtitles = true;
FAutoConsoleVariableRef CVarUseNewSubtitles(
	TEXT("au.UseNewSubtitles"),
	bUseNewSubtitles,
	TEXT("Defines whether to force DialogueWaves to use the new subtitles plugin. Currently-playing sounds will not have their queued or displayed subtitles updated.\n")
	TEXT("0: Use old subtitles\n")
	TEXT("1: Use new subtitles\n"),
	ECVF_Default
);

#if WITH_EDITORONLY_DATA
namespace
{
	// Class that takes DialogueWaves and prepares internationalization manifest entries 
	class FDialogueHelper
	{
	public:
		bool ProcessDialogueWave(const UDialogueWave* DialogueWave);
		const TArray<FTextSourceSiteContext>& GetContextSpecificVariations() const { return ContextSpecificVariations; }

		static void SetMetaDataFromContext(const UDialogueWave* DialogueWave, const FDialogueContextMapping& ContextMapping, FLocMetadataObject& OutInfoMetaData, FLocMetadataObject& OutKeyMetaData);

	private:
		static TSharedPtr<FLocMetadataValue> GetVoicesMetadata(const FString& SpeakerName, const TArray<FString>& TargetNames, bool bCompact = true);
		static FString GetDialogueVoiceName(const UDialogueVoice* DialogueVoice);
		static FString GetGrammaticalGenderString(const EGrammaticalGender::Type Gender);
		static FString GetGrammaticalNumberString(const EGrammaticalNumber::Type Plurality);

	private:
		// Context specific entries
		TArray<FTextSourceSiteContext> ContextSpecificVariations;
	};

	bool FDialogueHelper::ProcessDialogueWave(const UDialogueWave* DialogueWave)
	{
		if (!DialogueWave)
		{
			return false;
		}

		const FString SourceLocation = DialogueWave->GetPathName();

		for (const FDialogueContextMapping& ContextMapping : DialogueWave->ContextMappings)
		{
			const FDialogueContext& DialogueContext = ContextMapping.Context;

			// Skip over entries with invalid speaker
			if (!DialogueContext.Speaker)
			{
				continue;
			}

			FTextSourceSiteContext& Context = ContextSpecificVariations.AddDefaulted_GetRef();

			// Setup the variation context
			Context.KeyName = DialogueWave->GetContextLocalizationKey(ContextMapping);
			Context.SiteDescription = SourceLocation;
			Context.IsOptional = false;
			SetMetaDataFromContext(DialogueWave, ContextMapping, Context.InfoMetaData, Context.KeyMetaData);
		}

		return true;
	}

	void FDialogueHelper::SetMetaDataFromContext(const UDialogueWave* DialogueWave, const FDialogueContextMapping& ContextMapping, FLocMetadataObject& OutInfoMetaData, FLocMetadataObject& OutKeyMetaData)
	{
		const FDialogueContext& DialogueContext = ContextMapping.Context;
		const UDialogueVoice* SpeakerDialogueVoice = DialogueContext.Speaker;

		check(SpeakerDialogueVoice);

		// Collect speaker info
		const FString SpeakerDisplayName = GetDialogueVoiceName(SpeakerDialogueVoice);
		const FString SpeakerGender = GetGrammaticalGenderString(SpeakerDialogueVoice->Gender);
		const FString SpeakerPlurality = GetGrammaticalNumberString(SpeakerDialogueVoice->Plurality);
		const FString SpeakerGuid = SpeakerDialogueVoice->LocalizationGUID.ToString();

		TOptional<EGrammaticalGender::Type> AccumulatedTargetGender;
		TOptional<EGrammaticalNumber::Type> AccumulatedTargetPlurality;

		TArray<FString> TargetGuidsList;
		TArray<FString> TargetDisplayNameList;

		// Collect info on all the targets
		for (const UDialogueVoice* TargetDialogueVoice : DialogueContext.Targets)
		{
			if (TargetDialogueVoice)
			{
				const FString TargetDisplayName = GetDialogueVoiceName(TargetDialogueVoice);
				const FString TargetGender = GetGrammaticalGenderString(TargetDialogueVoice->Gender);
				const FString TargetPlurality = GetGrammaticalNumberString(TargetDialogueVoice->Plurality);
				const FString TargetGuid = TargetDialogueVoice->LocalizationGUID.ToString();

				TargetDisplayNameList.AddUnique(TargetDisplayName);
				TargetGuidsList.AddUnique(TargetGuid);

				if (!AccumulatedTargetGender.IsSet())
				{
					AccumulatedTargetGender = TargetDialogueVoice->Gender;
				}
				else if (AccumulatedTargetGender.GetValue() != TargetDialogueVoice->Gender)
				{
					AccumulatedTargetGender = EGrammaticalGender::Mixed;
				}

				if (!AccumulatedTargetPlurality.IsSet())
				{
					AccumulatedTargetPlurality = TargetDialogueVoice->Plurality;
				}
				else if (AccumulatedTargetPlurality.GetValue() == EGrammaticalNumber::Singular)
				{
					AccumulatedTargetPlurality = EGrammaticalNumber::Plural;
				}
			}
		}

		const FString FinalTargetGender = AccumulatedTargetGender.IsSet() ? GetGrammaticalGenderString(AccumulatedTargetGender.GetValue()) : TEXT("");
		const FString FinalTargetPlurality = AccumulatedTargetPlurality.IsSet() ? GetGrammaticalNumberString(AccumulatedTargetPlurality.GetValue()) : TEXT("");

		// Setup a loc metadata object with all the context specific keys.
		{
			if (!SpeakerGender.IsEmpty())
			{
				OutKeyMetaData.SetStringField(FDialogueConstants::PropertyName_GrammaticalGender, SpeakerGender);
			}

			if (!SpeakerPlurality.IsEmpty())
			{
				OutKeyMetaData.SetStringField(FDialogueConstants::PropertyName_GrammaticalPlurality, SpeakerPlurality);
			}

			if (!SpeakerGuid.IsEmpty())
			{
				OutKeyMetaData.SetStringField(FDialogueConstants::PropertyName_Speaker, SpeakerGuid);
			}

			if (!FinalTargetGender.IsEmpty())
			{
				OutKeyMetaData.SetStringField(FDialogueConstants::PropertyName_TargetGrammaticalGender, FinalTargetGender);
			}

			if (!FinalTargetPlurality.IsEmpty())
			{
				OutKeyMetaData.SetStringField(FDialogueConstants::PropertyName_TargetGrammaticalNumber, FinalTargetPlurality);
			}

			{
				TArray<TSharedPtr<FLocMetadataValue>> TargetGuidsMetadata;
				for (const FString& TargetGuid : TargetGuidsList)
				{
					TargetGuidsMetadata.Add(MakeShareable(new FLocMetadataValueString(TargetGuid)));
				}

				if (TargetGuidsMetadata.Num() > 0)
				{
					OutKeyMetaData.SetArrayField(FDialogueConstants::PropertyName_Targets, TargetGuidsMetadata);
				}
			}
		}

		// Setup a loc metadata object with all the context specific info.  This usually includes human readable descriptions of the dialogue
		{
			// Create the human readable info that describes the source and target voices of this dialogue
			{
				TSharedPtr<FLocMetadataValue> VoicesMetadata = GetVoicesMetadata(SpeakerDisplayName, TargetDisplayNameList);
				if (VoicesMetadata.IsValid())
				{
					OutInfoMetaData.SetField(FDialogueConstants::PropertyName_DialogueContext, VoicesMetadata);
				}
			}

			if (!DialogueWave->VoiceActorDirection.IsEmpty())
			{
				OutInfoMetaData.SetStringField(FDialogueConstants::PropertyName_VoiceActorDirection, DialogueWave->VoiceActorDirection);
			}

			{
				const FString AudioFile = DialogueWave->GetContextRecordedAudioFilename(ContextMapping);
				if (!AudioFile.IsEmpty())
				{
					OutInfoMetaData.SetStringField(FDialogueConstants::PropertyName_AudioFile, AudioFile);
				}
			}

			//OutInfoMetaData.SetBoolField(FDialogueConstants::PropertyName_IsMature, DialogueWave->bIsMature);
		}
	}

	TSharedPtr<FLocMetadataValue> FDialogueHelper::GetVoicesMetadata(const FString& SpeakerName, const TArray<FString>& TargetNames, bool bCompact)
	{
		/*
		This function can support two different formats.

		The first format is compact and results in string entries that will later be combined into something like this
		"Variations": [
			"Jenny -> Audience",
			"Zak -> Audience"
		]

		The second format is verbose and results in object entries that will later be combined into something like this
		"VariationsTest": [
			{
				"Speaker": "Jenny",
				"Targets": [
					"Audience"
					]
			},
			{
				"Speaker": "Zak",
				"Targets": [
					"Audience"
					]
			}
		]
		*/

		TSharedPtr<FLocMetadataValue> Result;
		if (bCompact)
		{
			TArray<FString> SortedTargetNames = TargetNames;
			SortedTargetNames.Sort();
			FString TargetNamesString;
			for (const FString& StrEntry : SortedTargetNames)
			{
				if (TargetNamesString.Len())
				{
					TargetNamesString += TEXT(",");
				}
				TargetNamesString += StrEntry;
			}
			Result = MakeShareable(new FLocMetadataValueString(FString::Printf(TEXT("%s -> %s"), *SpeakerName, *TargetNamesString)));
		}
		else
		{
			TArray<TSharedPtr<FLocMetadataValue>> TargetNamesMetadataList;
			for (const FString& StrEntry : TargetNames)
			{
				TargetNamesMetadataList.Add(MakeShareable(new FLocMetadataValueString(StrEntry)));
			}

			TSharedPtr<FLocMetadataObject> MetadataObj = MakeShareable(new FLocMetadataObject());
			MetadataObj->SetStringField(FDialogueConstants::PropertyName_Speaker, SpeakerName);
			MetadataObj->SetArrayField(FDialogueConstants::PropertyName_Targets, TargetNamesMetadataList);

			Result = MakeShareable(new FLocMetadataValueObject(MetadataObj.ToSharedRef()));
		}
		return Result;
	}

	FString FDialogueHelper::GetDialogueVoiceName(const UDialogueVoice* DialogueVoice)
	{
		return DialogueVoice->GetName();
	}

	FString FDialogueHelper::GetGrammaticalGenderString(const EGrammaticalGender::Type Gender)
	{
		switch (Gender)
		{
		case EGrammaticalGender::Neuter:
			return TEXT("Neuter");
		case EGrammaticalGender::Masculine:
			return TEXT("Masculine");
		case EGrammaticalGender::Feminine:
			return TEXT("Feminine");
		case EGrammaticalGender::Mixed:
			return TEXT("Mixed");
		default:
			return FString();
		}
	}

	FString FDialogueHelper::GetGrammaticalNumberString(const EGrammaticalNumber::Type Plurality)
	{
		switch (Plurality)
		{
		case EGrammaticalNumber::Singular:
			return TEXT("Singular");
		case EGrammaticalNumber::Plural:
			return TEXT("Plural");
		default:
			return FString();
		}
	}

	void GatherDialogueWaveForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
	{
		const UDialogueWave* const DialogueWave = CastChecked<UDialogueWave>(Object);

		PropertyLocalizationDataGatherer.GatherLocalizationDataFromObject(DialogueWave, GatherTextFlags);

		FDialogueHelper DialogueHelper;
		if (DialogueHelper.ProcessDialogueWave(DialogueWave))
		{
			auto FindOrAddDialogueTextData = [&PropertyLocalizationDataGatherer](const FString& InText, const FString& InNamespace) -> FGatherableTextData&
			{
				check(!InText.IsEmpty());

				FTextSourceData SourceData;
				SourceData.SourceString = InText;

				auto& GatherableTextDataArray = PropertyLocalizationDataGatherer.GetGatherableTextDataArray();
				FGatherableTextData* GatherableTextData = GatherableTextDataArray.FindByPredicate([&](const FGatherableTextData& Candidate)
				{
					return Candidate.NamespaceName.Equals(InNamespace, ESearchCase::CaseSensitive)
						&& Candidate.SourceData.SourceString.Equals(SourceData.SourceString, ESearchCase::CaseSensitive)
						&& Candidate.SourceData.SourceStringMetaData == SourceData.SourceStringMetaData;
				});
				if (!GatherableTextData)
				{
					GatherableTextData = &GatherableTextDataArray.AddDefaulted_GetRef();
					GatherableTextData->NamespaceName = InNamespace;
					GatherableTextData->SourceData = SourceData;
				}

				return *GatherableTextData;
			};

			// Gather the Spoken Text for each context
			if (!DialogueWave->SpokenText.IsEmpty())
			{
				FGatherableTextData& GatherableTextData = FindOrAddDialogueTextData(DialogueWave->SpokenText, FDialogueConstants::DialogueNamespace);

				const TArray<FTextSourceSiteContext>& Variations = DialogueHelper.GetContextSpecificVariations();
				for (const FTextSourceSiteContext& Variation : Variations)
				{
					GatherableTextData.SourceSiteContexts.Add(Variation);
				}
			}

			// Gather the Subtitle Override for each context
			if (DialogueWave->bOverride_SubtitleOverride && !DialogueWave->SubtitleOverride.IsEmpty())
			{
				FGatherableTextData& GatherableTextData = FindOrAddDialogueTextData(DialogueWave->SubtitleOverride, FDialogueConstants::DialogueNamespace);

				const TArray<FTextSourceSiteContext>& Variations = DialogueHelper.GetContextSpecificVariations();
				for (const FTextSourceSiteContext& Variation : Variations)
				{
					FTextSourceSiteContext SubtitleVariation = Variation;
					SubtitleVariation.KeyName += FDialogueConstants::SubtitleKeySuffix;
					SubtitleVariation.InfoMetaData.RemoveField(FDialogueConstants::PropertyName_AudioFile);
					GatherableTextData.SourceSiteContexts.Add(SubtitleVariation);
				}
			}

			// Gather the Voice Acting Direction
			if (!DialogueWave->VoiceActorDirection.IsEmpty())
			{
				FGatherableTextData& GatherableTextData = FindOrAddDialogueTextData(DialogueWave->VoiceActorDirection, FDialogueConstants::DialogueNotesNamespace);

				FTextSourceSiteContext& SourceSiteContext = GatherableTextData.SourceSiteContexts.AddDefaulted_GetRef();
				SourceSiteContext.KeyName = DialogueWave->LocalizationGUID.ToString() + FDialogueConstants::ActingDirectionKeySuffix;
				SourceSiteContext.SiteDescription = DialogueWave->GetPathName();
				SourceSiteContext.IsEditorOnly = true;
				SourceSiteContext.IsOptional = false;
			}
		}
	}
}
#endif //WITH_EDITORONLY_DATA

bool operator==(const FDialogueContextMapping& LHS, const FDialogueContextMapping& RHS)
{
	return	LHS.Context == RHS.Context &&
		LHS.SoundWave == RHS.SoundWave;
}

bool operator!=(const FDialogueContextMapping& LHS, const FDialogueContextMapping& RHS)
{
	return	LHS.Context != RHS.Context ||
		LHS.SoundWave != RHS.SoundWave;
}

FDialogueContextMapping::FDialogueContextMapping()
	: SoundWave(nullptr)
	, LocalizationKeyFormat(TEXT("{ContextHash}"))
	, Proxy(nullptr)
{

}

FString FDialogueContextMapping::GetLocalizationKey() const
{
	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("ContextHash"), Context.GetContextHash());
	return FString::Format(*LocalizationKeyFormat, Args);
}

FString FDialogueContextMapping::GetLocalizationKey(const FString& InOwnerDialogueWaveKey) const
{
	const FString ContextKey = GetLocalizationKey();
	return FString::Printf(TEXT("%s_%s"), *InOwnerDialogueWaveKey, *ContextKey);
}

UDialogueSoundWaveProxy::UDialogueSoundWaveProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UDialogueSoundWaveProxy::IsPlayable() const
{
	return SoundWave->IsPlayable();
}

const FSoundAttenuationSettings* UDialogueSoundWaveProxy::GetAttenuationSettingsToApply() const
{
	return SoundWave->GetAttenuationSettingsToApply();
}

float UDialogueSoundWaveProxy::GetMaxDistance() const
{
	return SoundWave->GetMaxDistance();
}

float UDialogueSoundWaveProxy::GetDuration() const
{
	return SoundWave->GetDuration();
}

float UDialogueSoundWaveProxy::GetVolumeMultiplier()
{
	return SoundWave->GetVolumeMultiplier();
}

float UDialogueSoundWaveProxy::GetPitchMultiplier()
{
	return SoundWave->GetPitchMultiplier();
}

void UDialogueSoundWaveProxy::Parse(class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	int OldWaveInstanceCount = WaveInstances.Num();
	const bool bHasSubtitles = bUseNewSubtitles ? !SubtitleText.IsEmpty() : Subtitles.Num() > 0;

	// Check if the wave instance exists before we try to add it...
	const bool bWaveInstanceAlreadyExisted = ActiveSound.FindWaveInstance(NodeWaveInstanceHash) != nullptr;

	ActiveSound.bHasExternalSubtitles = bHasSubtitles; // Need to set this so the sound will virtualize when silent if necessary.
	SoundWave->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, ParseParams, WaveInstances);
	int NewWaveInstanceCount = WaveInstances.Num();

	FWaveInstance* NewWaveInstance = nullptr;
	if (NewWaveInstanceCount == OldWaveInstanceCount + 1)
	{
		NewWaveInstance = WaveInstances[WaveInstances.Num() - 1];
	}

	// Only Queue subtitles once per each playback of a wave instance
	if (NewWaveInstance != nullptr && !bWaveInstanceAlreadyExisted)
	{
		// Add in the subtitle if they exist
		if (ActiveSound.bHandleSubtitles && bHasSubtitles)
		{
			if (!bUseNewSubtitles)
			{
				FQueueSubtitleParams QueueSubtitleParams(Subtitles);
				{
					QueueSubtitleParams.AudioComponentID = ActiveSound.GetAudioComponentID();
					QueueSubtitleParams.WorldPtr = ActiveSound.GetWeakWorld();
					QueueSubtitleParams.WaveInstance = (PTRINT)NewWaveInstance;
					QueueSubtitleParams.SubtitlePriority = ActiveSound.SubtitlePriority;
					QueueSubtitleParams.Duration = GetDuration();
					QueueSubtitleParams.bManualWordWrap = false;
					QueueSubtitleParams.bSingleLine = false;
					QueueSubtitleParams.RequestedStartTime = ActiveSound.RequestedStartTime;
				}

				FSubtitleManager::QueueSubtitles(QueueSubtitleParams);
			}
		}
	}
}

USoundClass* UDialogueSoundWaveProxy::GetSoundClass() const
{
	return SoundWave->GetSoundClass();
}


UCurveTable* UDialogueSoundWaveProxy::GetCurveData() const
{
	return SoundWave ? SoundWave->GetCurveData() : nullptr;
}

UDialogueWave::UDialogueWave(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LocalizationGUID(IsTemplate() ? FGuid() : FGuid::NewGuid())
{
#if WITH_EDITORONLY_DATA
	{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(UDialogueWave::StaticClass(), &GatherDialogueWaveForLocalization); }
#endif

	bOverride_SubtitleOverride = false;
	ContextMappings.Add(FDialogueContextMapping());
}

// Begin UObject interface. 
void UDialogueWave::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.ThisRequiresLocalizationGather();

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogAudio, Fatal, "This platform requires cooked packages, and audio data was not cooked into %ls.", *GetFullName());
	}
}

bool UDialogueWave::IsReadyForFinishDestroy()
{
	return true;
}

FString UDialogueWave::GetDesc()
{
	return FString::Printf(TEXT("Dialogue Wave Description"));
}

void UDialogueWave::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UDialogueWave::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
}

void UDialogueWave::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if (!bDuplicateForPIE)
	{
		LocalizationGUID = FGuid::NewGuid();
	}
}

void UDialogueWave::PostLoad()
{
	Super::PostLoad();

	for (FDialogueContextMapping& ContextMapping : ContextMappings)
	{
		UpdateMappingProxy(ContextMapping);
	}
}

#if WITH_EDITOR
void UDialogueWave::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	int32 NewArrayIndex = PropertyChangedEvent.GetArrayIndex(TEXT("ContextMappings"));

	if (ContextMappings.IsValidIndex(NewArrayIndex))
	{
		UpdateMappingProxy(ContextMappings[NewArrayIndex]);
	}
	else if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UDialogueWave, SpokenText))
	{
		for (FDialogueContextMapping& ContextMapping : ContextMappings)
		{
			UpdateMappingProxy(ContextMapping);
		}
	}
}
#endif

// End UObject interface. 

// Begin UDialogueWave interface.
bool UDialogueWave::SupportsContext(const FDialogueContext& Context) const
{
	for (const FDialogueContextMapping& ContextMapping : ContextMappings)
	{
		if (ContextMapping.Context == Context)
		{
			return true;
		}
	}

	return false;
}

USoundBase* UDialogueWave::GetWaveFromContext(const FDialogueContext& Context) const
{
	if (Context.Speaker == nullptr)
	{
		UE_LOGF(LogAudio, Warning, "UDialogueWave::GetWaveFromContext requires a Context.Speaker (%ls).", *GetPathName());
		return nullptr;
	}

	UDialogueSoundWaveProxy* Proxy = nullptr;
	for (const FDialogueContextMapping& ContextMapping : ContextMappings)
	{
		if (ContextMapping.Context == Context)
		{
			Proxy = ContextMapping.Proxy;
			break;
		}
	}

	return Proxy;
}

USoundBase* UDialogueWave::GetWaveFromContext(const FDialogueContextMapping& ContextMapping) const
{
	if (ContextMapping.Context.Speaker == nullptr)
	{
		UE_LOGF(LogAudio, Warning, "UDialogueWave::GetWaveFromContext requires a Context.Speaker (%ls).", *GetPathName());
		return nullptr;
	}

	return ContextMapping.Proxy;
}

FText UDialogueWave::GetLocalizedSpokenText() const
{
	// Try and find a general dialogue wave localization
	return FText::AsLocalizable_Advanced(FDialogueConstants::DialogueNamespace, LocalizationGUID.ToString(), SpokenText);
}

FText UDialogueWave::GetLocalizedSpokenText(const FDialogueContext& Context) const
{
	for (const FDialogueContextMapping& ContextMapping : ContextMappings)
	{
		if (ContextMapping.Context == Context)
		{
			return GetLocalizedSpokenText(ContextMapping);
		}
	}
	return FText::GetEmpty();
}

FText UDialogueWave::GetLocalizedSpokenText(const FDialogueContextMapping& ContextMapping) const
{
	// First try and find a context specific localization
	if (ContextMapping.Context.Speaker)
	{
		FText ContextSpecificText;
		if (FText::FindTextInLiveTable_Advanced(FDialogueConstants::DialogueNamespace, GetContextLocalizationKey(ContextMapping), ContextSpecificText, &SpokenText))
		{
			return ContextSpecificText;
		}
	}

	// Failing that, try and find a general dialogue wave localization
	return GetLocalizedSpokenText();
}

FText UDialogueWave::GetLocalizedSubtitle() const
{
	FStringView SubtitleSourceString = SpokenText;

	// Try and find a general dialogue wave localization
	FString Key = LocalizationGUID.ToString();
	if (bOverride_SubtitleOverride)
	{
		SubtitleSourceString = SubtitleOverride;
		Key += FDialogueConstants::SubtitleKeySuffix;
	}

	return FText::AsLocalizable_Advanced(FDialogueConstants::DialogueNamespace, Key, SubtitleSourceString);
}

FText UDialogueWave::GetLocalizedSubtitle(const FDialogueContext& Context) const
{
	for (const FDialogueContextMapping& ContextMapping : ContextMappings)
	{
		if (ContextMapping.Context == Context)
		{
			return GetLocalizedSubtitle(ContextMapping);
		}
	}
	return FText::GetEmpty();
}

FText UDialogueWave::GetLocalizedSubtitle(const FDialogueContextMapping& ContextMapping) const
{
	// First try and find a context specific localization
	if (ContextMapping.Context.Speaker)
	{
		const FString* SubtitleSourceString = &SpokenText;

		// Do we have a subtitle override?
		FString Key = GetContextLocalizationKey(ContextMapping);
		if (bOverride_SubtitleOverride)
		{
			SubtitleSourceString = &SubtitleOverride;
			Key += FDialogueConstants::SubtitleKeySuffix;
		}

		FText ContextSpecificText;
		if (FText::FindTextInLiveTable_Advanced(FDialogueConstants::DialogueNamespace, Key, ContextSpecificText, SubtitleSourceString))
		{
			return ContextSpecificText;
		}
	}

	// Failing that, try and find a general dialogue wave localization
	return GetLocalizedSubtitle();
}

FString UDialogueWave::GetContextLocalizationKey(const FDialogueContext& Context) const
{
	for (const FDialogueContextMapping& ContextMapping : ContextMappings)
	{
		if (ContextMapping.Context == Context)
		{
			return GetContextLocalizationKey(ContextMapping);
		}
	}
	return FString();
}

FString UDialogueWave::GetContextLocalizationKey(const FDialogueContextMapping& ContextMapping) const
{
	return ContextMapping.GetLocalizationKey(LocalizationGUID.ToString());
}

FString UDialogueWave::GetContextRecordedAudioFilename(const FDialogueContext& Context) const
{
	for (const FDialogueContextMapping& ContextMapping : ContextMappings)
	{
		if (ContextMapping.Context == Context)
		{
			return GetContextRecordedAudioFilename(ContextMapping);
		}
	}
	return FString();
}

FString UDialogueWave::GetContextRecordedAudioFilename(const FDialogueContextMapping& ContextMapping) const
{
	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	const FString DialogueContextFilename = BuildRecordedAudioFilename(AudioSettings->DialogueFilenameFormat, LocalizationGUID, GetName(), ContextMapping.GetLocalizationKey(), ContextMappings.IndexOfByKey(ContextMapping));
	return FString::Printf(TEXT("%s.wav"), *DialogueContextFilename);
}

FString UDialogueWave::BuildRecordedAudioFilename(const FString& FormatString, const FGuid& DialogueGuid, const FString& DialogueName, const FString& ContextId, const int32 ContextIndex)
{
	static const FString DialogueWaveSuffix = "_DialogueWave";
	static const FString DialogueSuffix = "_Dialogue";

	const FString DialogueHash = FString::Printf(TEXT("%08X"), FCrc::MemCrc32(&DialogueGuid, sizeof(FGuid)));

	// Trim the asset name if it ends with a common leaf
	FString TrimmedDialogueName = DialogueName;
	if (!TrimmedDialogueName.RemoveFromEnd(DialogueWaveSuffix))
	{
		TrimmedDialogueName.RemoveFromEnd(DialogueSuffix);
	}

	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("DialogueGuid"), DialogueGuid.ToString());
	Args.Add(TEXT("DialogueHash"), DialogueHash);
	Args.Add(TEXT("DialogueName"), TrimmedDialogueName);
	Args.Add(TEXT("ContextId"), ContextId);
	Args.Add(TEXT("ContextIndex"), ContextIndex);
	
	return FString::Format(*FormatString, Args);
}

// End UDialogueWave interface.

void UDialogueWave::UpdateContext(FDialogueContextMapping& ContextMapping, USoundWave* SoundWave, UDialogueVoice* Speaker, const TArray<UDialogueVoice*>& Targets)
{
	ContextMapping.SoundWave = SoundWave;
	ContextMapping.Context.Speaker = Speaker;
	ContextMapping.Context.Targets = Targets;

	UpdateMappingProxy(ContextMapping);
}

void UDialogueWave::UpdateMappingProxy(FDialogueContextMapping& ContextMapping)
{
	if (ContextMapping.SoundWave)
	{
		if (!ContextMapping.Proxy)
		{
			ContextMapping.Proxy = NewObject<UDialogueSoundWaveProxy>();
		}
	}
	else
	{
		ContextMapping.Proxy = nullptr;
	}

	if (ContextMapping.Proxy)
	{
		// Copy the properties that the proxy shares with the sound in case it's used as a SoundBase
		ContextMapping.Proxy->SoundWave = ContextMapping.SoundWave;
		CopySoundBasePropertiesToProxy(ContextMapping.SoundWave, ContextMapping.Proxy);

		// Set up new subtitles...
		ContextMapping.Proxy->SubtitleText = GetLocalizedSubtitle(ContextMapping);
		// ...and legacy subtitles.
		// This is to make sure both types are properly initialized if the migration cvar au.UseNewSubtitles is swapped while the game is running.
		FSubtitleCue NewSubtitleCue;
		NewSubtitleCue.Text = GetLocalizedSubtitle(ContextMapping);
		NewSubtitleCue.Time = 0.0f;

		ContextMapping.Proxy->Subtitles.Empty();
		ContextMapping.Proxy->Subtitles.Add(NewSubtitleCue);
	}
}

void UDialogueWave::CopySoundBasePropertiesToProxy(const USoundBase* InSoundBase, USoundBase* Proxy)
{
	Proxy->SoundClassObject = InSoundBase->SoundClassObject;
	Proxy->bDebug = InSoundBase->bDebug;
	Proxy->bOverrideConcurrency = InSoundBase->bOverrideConcurrency;
	Proxy->bEnableBusSends = InSoundBase->bEnableBusSends;
	Proxy->bEnableBaseSubmix = InSoundBase->bEnableBaseSubmix;
	Proxy->bEnableSubmixSends = InSoundBase->bEnableSubmixSends;
	Proxy->bHasDelayNode = InSoundBase->bHasDelayNode;
	Proxy->bHasConcatenatorNode = InSoundBase->bHasConcatenatorNode;
	Proxy->VirtualizationMode = InSoundBase->VirtualizationMode;
	Proxy->bBypassVolumeScaleForPriority = InSoundBase->bBypassVolumeScaleForPriority;
	Proxy->ConcurrencySet = InSoundBase->ConcurrencySet;
	Proxy->ConcurrencyOverrides = InSoundBase->ConcurrencyOverrides;
	Proxy->Duration = InSoundBase->Duration;
	Proxy->MaxDistance = InSoundBase->MaxDistance;
	Proxy->TotalSamples = InSoundBase->TotalSamples;
	Proxy->Priority = InSoundBase->Priority;
	Proxy->AttenuationSettings = InSoundBase->AttenuationSettings;
	Proxy->SoundSubmixObject = InSoundBase->SoundSubmixObject;
	Proxy->SoundSubmixSends = InSoundBase->SoundSubmixSends;
	Proxy->SourceEffectChain = InSoundBase->SourceEffectChain;
	Proxy->BusSends = InSoundBase->BusSends;
	Proxy->PreEffectBusSends = InSoundBase->PreEffectBusSends;
}

