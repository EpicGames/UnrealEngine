// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldAtlas.cpp
=============================================================================*/

#include "DistanceFieldAtlas.h"
#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Texture2D.h"
#include "EngineLogs.h"
#include "StaticMeshResources.h"
#include "Materials/MaterialInterface.h"
#include "ProfilingDebugging/CookStats.h"
#include "RHIStaticStates.h"
#include "Engine/StaticMesh.h"
#include "Misc/AutomationTest.h"
#include "GlobalShader.h"
#include "PrimitiveDrawingUtils.h"
#include "TextureResource.h"
#include "MeshCardBuild.h"
#include "MeshCardRepresentation.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "ObjectCacheContext.h"
#include "RenderGraphUtils.h"
#include "Rendering/NaniteResources.h"

#if WITH_EDITOR
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheKey.h"
#include "StaticMeshCompiler.h"
#endif

#if WITH_EDITORONLY_DATA
#include "IMeshBuilderModule.h"
#endif

#include <atomic>

#define LOCTEXT_NAMESPACE "DistanceField"

#if ENABLE_COOK_STATS
namespace DistanceFieldCookStats
{
	FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("DistanceField.Usage"), TEXT(""));
	});
}
#endif

static TAutoConsoleVariable<int32> CVarDistField(
	TEXT("r.GenerateMeshDistanceFields"),
	0,	
	TEXT("Whether to build distance fields of static meshes, needed for Lumen Software Ray Tracing and Distance Field AO, which is used to implement Movable SkyLight shadows.\n")
	TEXT("Enabling will increase mesh build times and memory usage.  Changing this value will cause a rebuild of all static meshes."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDistFieldSupportWhenHardwareRayTracingEnabled(
	TEXT("r.DistanceFields.SupportEvenIfHardwareRayTracingSupported"),
	1,
	TEXT("Whether to support distance fields when hardware ray tracing is supported.\n")
	TEXT("Setting it to 0 will skip distance field overhead when hardware ray tracing is supported."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDistFieldRes(
	TEXT("r.DistanceFields.MaxPerMeshResolution"),
	256,	
	TEXT("Highest resolution (in one dimension) allowed for a single static mesh asset, used to cap the memory usage of meshes with a large scale.\n")
	TEXT("Changing this will cause all distance fields to be rebuilt.  Large values such as 512 can consume memory very quickly! (64Mb for one asset at 512)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<float> CVarDistFieldResScale(
	TEXT("r.DistanceFields.DefaultVoxelDensity"),
	.2f,	
	TEXT("Determines how the default scale of a mesh converts into distance field voxel dimensions.\n")
	TEXT("Changing this will cause all distance fields to be rebuilt.  Large values can consume memory very quickly!"),
	ECVF_ReadOnly);

static int32 GHeightFieldAtlasTileSize = 64;
static FAutoConsoleVariableRef CVarHeightFieldAtlasTileSize(
	TEXT("r.HeightFields.AtlasTileSize"),
	GHeightFieldAtlasTileSize,
	TEXT("Suballocation granularity"),
	ECVF_RenderThreadSafe);

static int32 GHeightFieldAtlasDimInTiles = 16;
static FAutoConsoleVariableRef CVarHeightFieldAtlasDimInTiles(
	TEXT("r.HeightFields.AtlasDimInTiles"),
	GHeightFieldAtlasDimInTiles,
	TEXT("Number of tiles the atlas has in one dimension"),
	ECVF_RenderThreadSafe);

static int32 GHeightFieldAtlasDownSampleLevel = 2;
static FAutoConsoleVariableRef CVarHeightFieldAtlasDownSampleLevel(
	TEXT("r.HeightFields.AtlasDownSampleLevel"),
	GHeightFieldAtlasDownSampleLevel,
	TEXT("Max number of times a suballocation can be down-sampled"),
	ECVF_RenderThreadSafe);

static int32 GHFVisibilityAtlasTileSize = 64;
static FAutoConsoleVariableRef CVarHFVisibilityAtlasTileSize(
	TEXT("r.HeightFields.VisibilityAtlasTileSize"),
	GHFVisibilityAtlasTileSize,
	TEXT("Suballocation granularity"),
	ECVF_RenderThreadSafe);

static int32 GHFVisibilityAtlasDimInTiles = 8;
static FAutoConsoleVariableRef CVarHFVisibilityAtlasDimInTiles(
	TEXT("r.HeightFields.VisibilityAtlasDimInTiles"),
	GHFVisibilityAtlasDimInTiles,
	TEXT("Number of tiles the atlas has in one dimension"),
	ECVF_RenderThreadSafe);

static int32 GHFVisibilityAtlasDownSampleLevel = 2;
static FAutoConsoleVariableRef CVarHFVisibilityAtlasDownSampleLevel(
	TEXT("r.HeightFields.VisibilityAtlasDownSampleLevel"),
	GHFVisibilityAtlasDownSampleLevel,
	TEXT("Max number of times a suballocation can be down-sampled"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarDistanceFieldsEstimate(
	TEXT("r.CookMemory.DistanceFieldsEstimate"), true,
	TEXT("Enable memory estimates for distance field asset cooks"),
	ECVF_ReadOnly
); 

TGlobalResource<FLandscapeTextureAtlas> GHeightFieldTextureAtlas(FLandscapeTextureAtlas::SAT_Height);
TGlobalResource<FLandscapeTextureAtlas> GHFVisibilityTextureAtlas(FLandscapeTextureAtlas::SAT_Visibility);

FDistanceFieldAsyncQueue* GDistanceFieldAsyncQueue = NULL;

#if WITH_EDITOR

FAsyncCardRepresentationTask* BeginCacheMeshCardRepresentationInternal(
	UStaticMesh* StaticMeshAsset,
	FStaticMeshRenderData& RenderData,
	const FString& DistanceFieldKey,
	FSourceMeshDataForDerivedDataTask&& OptionalSourceMeshData,
	TArray<FSignedDistanceFieldBuildPartData>&& OptionalPartData,
	FDistanceFieldVolumeData* DistanceFieldData);

// DDC key for distance field data, must be changed when modifying the generation code or data format
#define DISTANCEFIELD_DERIVEDDATA_VER TEXT("DC2427EE-AD20-4226-ADAD-15CAEB4FC9AB")

void AppendDistanceFieldConstantsToClassSchema(FAppendToClassSchemaContext& Context)
{
	IConsoleManager& CM = IConsoleManager::Get();
	const int32 PerMeshMax
		= CM.FindTConsoleVariableDataInt(TEXT("r.DistanceFields.MaxPerMeshResolution"))->GetValueOnAnyThread();
	const float VoxelDensity
		= CM.FindTConsoleVariableDataFloat(TEXT("r.DistanceFields.DefaultVoxelDensity"))->GetValueOnAnyThread();
	FStringView VersionString(DISTANCEFIELD_DERIVEDDATA_VER);

	Context.Update(&PerMeshMax, sizeof(PerMeshMax));
	Context.Update(&VoxelDensity, sizeof(VoxelDensity));
	Context.Update(VersionString.GetData(), VersionString.Len() * sizeof(VersionString[0]));
}

FString BuildDistanceFieldDerivedDataKey(const FString& InMeshKey)
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DistanceFields.MaxPerMeshResolution"));
	const int32 PerMeshMax = CVar->GetValueOnAnyThread();
	const FString PerMeshMaxString = PerMeshMax == 128 ? TEXT("") : FString::Printf(TEXT("_%u"), PerMeshMax);

	static const auto CVarDensity = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.DistanceFields.DefaultVoxelDensity"));
	const float VoxelDensity = CVarDensity->GetValueOnAnyThread();
	const FString VoxelDensityString = VoxelDensity == .1f ? TEXT("") : FString::Printf(TEXT("_%.3f"), VoxelDensity);

	static UE::DerivedData::FCacheBucket LegacyBucket(TEXTVIEW("LegacyDIST"), TEXTVIEW("DistanceField"));
	return FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("DIST"),
		*FString::Printf(TEXT("%s_%s%s%s"), *InMeshKey, DISTANCEFIELD_DERIVEDDATA_VER, *PerMeshMaxString, *VoxelDensityString),
		TEXT(""));
}

#endif

#if WITH_EDITORONLY_DATA

void BuildSignedDistanceFieldBuildSectionData(
	const FMeshSectionInfoMap& SectionInfoMap,
	uint32 LODIndex,
	const TArray<FStaticMaterial>& StaticMaterials,
	const TArray<int32>& MaterialRemap,
	const FStaticMeshRenderData& RenderData,
	TArray<FSignedDistanceFieldBuildSectionData>& OutData)
{
	OutData.SetNum(SectionInfoMap.GetSectionNumber(LODIndex));

	for (int32 SectionIndex = 0; SectionIndex < OutData.Num(); SectionIndex++)
	{
		const FMeshSectionInfo& Section = SectionInfoMap.Get(LODIndex, SectionIndex);

		FSignedDistanceFieldBuildSectionData& SectionData = OutData[SectionIndex];
		SectionData.bAffectDistanceFieldLighting = Section.bAffectDistanceFieldLighting;

		int32 MaterialIndex = Section.MaterialIndex;

		if (!MaterialRemap.IsEmpty())
		{
			MaterialIndex = MaterialRemap.IsValidIndex(MaterialIndex) ? MaterialRemap[MaterialIndex] : 0;
		}

		if (!StaticMaterials.IsValidIndex(MaterialIndex))
		{
			// TODO: Should maybe log warning here
			continue;
		}

		UMaterialInterface* MaterialInterface = StaticMaterials[MaterialIndex].MaterialInterface;
		if (MaterialInterface)
		{
			SectionData.BlendMode = MaterialInterface->GetBlendMode();
			SectionData.bTwoSided = MaterialInterface->IsTwoSided();
		}

		if (RenderData.NaniteResourcesPtr.IsValid())
		{
			SectionData.bTwoSided |= (RenderData.NaniteResourcesPtr->VoxelMaterialsMask & (1ull << MaterialIndex)) != 0;
		}
	}
}

void FDistanceFieldVolumeData::CacheDerivedData(const FString& InStaticMeshDerivedDataKey, const ITargetPlatform* TargetPlatform, UStaticMesh* Mesh, FStaticMeshRenderData& RenderData, UStaticMesh* GenerateSource, float DistanceFieldResolutionScale, bool bGenerateDistanceFieldAsIfTwoSided)
{
	CacheDerivedData(Mesh, GenerateSource, DistanceFieldResolutionScale, bGenerateDistanceFieldAsIfTwoSided);
}

void FDistanceFieldVolumeData::CacheDerivedData(UStaticMesh* Mesh, UStaticMesh* GenerateSource, float DistanceFieldResolutionScale, bool bGenerateDistanceFieldAsIfTwoSided)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldVolumeData::CacheDerivedData);
	COOK_STAT(auto Timer = DistanceFieldCookStats::UsageStats.TimeSyncWork());

	TArray<FSignedDistanceFieldBuildSectionData> BuildSectionData;

	const uint32 LODIndex = 0;

	BuildSignedDistanceFieldBuildSectionData(Mesh->GetSectionInfoMap(), LODIndex, Mesh->GetStaticMaterials(), {}, *Mesh->GetRenderData(), BuildSectionData);

	FString DistanceFieldKey = BuildDistanceFieldDerivedDataKey(Mesh->GetRenderData()->DerivedDataKey);

	for (int32 SectionIndex = 0; SectionIndex < BuildSectionData.Num(); SectionIndex++)
	{
		DistanceFieldKey += FString::Printf(TEXT("_M%u_%u_%u"), 
			(uint32)BuildSectionData[SectionIndex].BlendMode,
			BuildSectionData[SectionIndex].bTwoSided ? 1 : 0,
			BuildSectionData[SectionIndex].bAffectDistanceFieldLighting ? 1 : 0);
	}

	TArray<FSignedDistanceFieldBuildPartData> PartDatas;

	if (Mesh->IsNaniteAssembly())
	{
		const TArray<TObjectPtr<UStaticMesh>>& PartReferences = Mesh->GetCachedNaniteAssemblyReferences();
		const int32 NumParts = PartReferences.Num();

		PartDatas.SetNum(NumParts);

		for (int32 PartIndex = 0; PartIndex < NumParts; ++PartIndex)
		{
			const TObjectPtr<UStaticMesh>& Ref = PartReferences[PartIndex];

			if (Ref)
			{
				FSignedDistanceFieldBuildPartData& PartData = PartDatas[PartIndex];

				const FMeshSectionInfoMap& SectionInfoMap = Ref->GetSectionInfoMap();
				const TArray<FStaticMaterial>& StaticMaterials = Mesh->GetStaticMaterials();
				const TArray<int32>& MaterialRemap = Mesh->GetNaniteSettings().NaniteAssemblyData.Parts[PartIndex].MaterialRemap;

				BuildSignedDistanceFieldBuildSectionData(SectionInfoMap, LODIndex, StaticMaterials, MaterialRemap, *Mesh->GetRenderData(), PartData.SectionData);

				// include part section data into DDC key
				for (int32 SectionIndex = 0; SectionIndex < PartData.SectionData.Num(); SectionIndex++)
				{
					DistanceFieldKey += FString::Printf(TEXT("_PM%u_%u_%u"),
						(uint32)PartData.SectionData[SectionIndex].BlendMode,
						PartData.SectionData[SectionIndex].bTwoSided ? 1 : 0,
						PartData.SectionData[SectionIndex].bAffectDistanceFieldLighting ? 1 : 0);
				}
			}
		}
	}

	TArray<uint8> DerivedData;

	if (GetDerivedDataCacheRef().GetSynchronous(*DistanceFieldKey, DerivedData, Mesh->GetPathName()))
	{
		COOK_STAT(Timer.AddHit(DerivedData.Num()));
		FMemoryReader Ar(DerivedData, /*bIsPersistent=*/ true);
		Serialize(Ar, Mesh);

		BeginCacheMeshCardRepresentation(Mesh, *Mesh->GetRenderData(), DistanceFieldKey);
	}
	else if (GDistanceFieldAsyncQueue)
	{
		check(Mesh && GenerateSource);

		// Mark that this resource is async building
		bAsyncBuilding = true;

		// We don't actually build the resource until later, so only track the cycles used here.
		COOK_STAT(Timer.TrackCyclesOnly());
		FAsyncDistanceFieldTask* NewTask = new FAsyncDistanceFieldTask;
		NewTask->DDCKey = DistanceFieldKey;

		// Always use GetRunningTargetPlatform() here (the cooker's platform, typically Windows Editor), as the data is intended to be platform independent.
		// Besides that, TargetPlatform is only used to fetch an IMeshBuilderModule interface, and in practice the BuildMeshVertexPositions function on
		// that interface doesn't vary per platform (all platforms internally use FStaticMeshBuilder::BuildMeshVertexPositions).
		NewTask->TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		NewTask->StaticMesh = Mesh;
		NewTask->GenerateSource = GenerateSource;
		NewTask->DistanceFieldResolutionScale = DistanceFieldResolutionScale;
		NewTask->bGenerateDistanceFieldAsIfTwoSided = bGenerateDistanceFieldAsIfTwoSided;
		NewTask->GeneratedVolumeData = new FDistanceFieldVolumeData();
		NewTask->GeneratedVolumeData->AssetName = Mesh->GetFName();
		NewTask->GeneratedVolumeData->bAsyncBuilding = true;
		NewTask->SectionData = MoveTemp(BuildSectionData);

		// Nanite overrides source static mesh with a coarse representation. Need to load original data before we build the mesh SDF.
		if (Mesh->IsNaniteEnabled())
		{
			IMeshBuilderModule& MeshBuilderModule = IMeshBuilderModule::GetForPlatform(NewTask->TargetPlatform);
			
			if (!MeshBuilderModule.BuildMeshVertexPositions(Mesh, NewTask->SourceMeshData.TriangleIndices, NewTask->SourceMeshData.VertexPositions, NewTask->SourceMeshData.Sections))
			{
				if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
				{
					delete NewTask;
					return;
				}

				UE_LOGF(LogStaticMesh, Error, "Failed to build static mesh. See previous line(s) for details.");
			}

			if (Mesh->IsNaniteAssembly())
			{
				check(Mesh->HasCachedNaniteAssemblyReferences())

				const TArray<TObjectPtr<UStaticMesh>>& PartReferences = Mesh->GetCachedNaniteAssemblyReferences();
				const int32 NumParts = PartReferences.Num();

				check(NumParts == PartDatas.Num());

				for (int32 Index = 0; Index < NumParts; ++Index)
				{
					const TObjectPtr<UStaticMesh>& Ref = PartReferences[Index];

					if (Ref)
					{
						FSignedDistanceFieldBuildPartData& PartData = PartDatas[Index];
						FSourceMeshDataForDerivedDataTask& SourceMeshData = PartData.SourceMeshData;

						if (!MeshBuilderModule.BuildMeshVertexPositions(Ref, SourceMeshData.TriangleIndices, SourceMeshData.VertexPositions, SourceMeshData.Sections))
						{
							UE_LOGF(LogStaticMesh, Error, "Failed to build static mesh. See previous line(s) for details.");
						}
					}
					else
					{
						UE_LOGF(LogStaticMesh, Warning, "Nanite Assembly doesn't have a valid part reference. See previous line(s) for details.");
					}
				}

				NewTask->PartData = PartDatas;
			}
		}

		GDistanceFieldAsyncQueue->AddTask(NewTask);
	}
}

#endif

std::atomic<uint64> NextDistanceFieldVolumeDataId { 1 };

FDistanceFieldVolumeData::FDistanceFieldVolumeData() :
	LocalSpaceMeshBounds(ForceInit),
	bMostlyTwoSided(false),
	bAsyncBuilding(false)
{
	Id = NextDistanceFieldVolumeDataId++;
}

void FDistanceFieldVolumeData::Serialize(FArchive& Ar, UObject* Owner)
{
	// Note: this is derived data, no need for versioning (bump the DDC guid)
	Ar << LocalSpaceMeshBounds << bMostlyTwoSided << Mips << AlwaysLoadedMip;
	StreamableMips.Serialize(Ar, Owner, 0);
}

int32 GUseAsyncDistanceFieldBuildQueue = 1;
static FAutoConsoleVariableRef CVarAOAsyncBuildQueue(
	TEXT("r.AOAsyncBuildQueue"),
	GUseAsyncDistanceFieldBuildQueue,
	TEXT("Whether to asynchronously build distance field volume data from meshes."),
	ECVF_Default | ECVF_ReadOnly
	);

FAsyncDistanceFieldTask::FAsyncDistanceFieldTask()
	: StaticMesh(nullptr)
	, GenerateSource(nullptr)
	, DistanceFieldResolutionScale(0.0f)
	, bGenerateDistanceFieldAsIfTwoSided(false)
	, GeneratedVolumeData(nullptr)
{
}

void FDistanceFieldAsyncQueue::OnAssetPostCompile(const TArray<FAssetCompileData>& CompiledAssets)
{
	for (const FAssetCompileData& CompileData : CompiledAssets)
	{
		if (CompileData.Asset.IsValid() && CompileData.Asset.Get()->IsA<UStaticMesh>())
		{
			const UStaticMesh* StaticMesh = Cast<UStaticMesh>(CompileData.Asset.Get());
			ProcessPendingTasksForMesh(StaticMesh);
		}
	}
}

#if WITH_EDITOR
void FDistanceFieldAsyncQueue::FinalizeAsyncTask(FAsyncDistanceFieldTask* InTask, FObjectCacheContextScope& InObjectCacheScope) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::FinalizeAsyncTask);

	check(InTask);
	check(IsInGameThread()); // Timer wont work properly outside of gamethread.

	// We want to count each resource built from a DDC miss, so count each iteration of the loop separately.
	COOK_STAT(auto Timer = DistanceFieldCookStats::UsageStats.TimeSyncWork());

	if (InTask->AsyncTask)
	{
		InTask->AsyncTask->EnsureCompletion();
		InTask->AsyncTask.Reset();
	}

	// Editor 'force delete' can null any UObject pointers which are seen by reference collecting (eg FProperty or serialized)
	if (InTask->StaticMesh)
	{
		check(!InTask->StaticMesh->IsCompiling());

		InTask->GeneratedVolumeData->bAsyncBuilding = false;

		FStaticMeshRenderData* RenderData = InTask->StaticMesh->GetRenderData();
		FDistanceFieldVolumeData* OldVolumeData = RenderData->LODResources[0].DistanceFieldData;

		// Assign the new volume data, this is safe because the render thread makes a copy of the pointer at scene proxy creation time.
		RenderData->LODResources[0].DistanceFieldData = InTask->GeneratedVolumeData;

		// Renderstates are not initialized between UStaticMesh::PreEditChange() and UStaticMesh::PostEditChange()
		if (RenderData->IsInitialized())
		{
			for (IStaticMeshComponent* Component : InObjectCacheScope.GetContext().GetStaticMeshComponents(InTask->StaticMesh))
			{
				IPrimitiveComponent* PrimitiveComponent = Component->GetPrimitiveComponentInterface();

				if (PrimitiveComponent->IsRegistered() && PrimitiveComponent->IsRenderStateCreated())
				{
					PrimitiveComponent->MarkRenderStateDirty();
				}
			}
		}

		if (OldVolumeData)
		{
			// Rendering thread may still be referencing the old one, use the deferred cleanup interface to delete it next frame when it is safe
			BeginCleanup(OldVolumeData);
		}

		// Need also to update platform render data if it's being cached
		FStaticMeshRenderData* PlatformRenderData = RenderData->NextCachedRenderData.Get();
		while (PlatformRenderData)
		{
			if (PlatformRenderData->LODResources[0].DistanceFieldData)
			{
				*PlatformRenderData->LODResources[0].DistanceFieldData = *InTask->GeneratedVolumeData;
				// The old bulk data assignment operator doesn't copy over flags
				PlatformRenderData->LODResources[0].DistanceFieldData->StreamableMips.ResetBulkDataFlags(InTask->GeneratedVolumeData->StreamableMips.GetBulkDataFlags());
			}
			PlatformRenderData = PlatformRenderData->NextCachedRenderData.Get();
		}

		{
			TArray<uint8> DerivedData;
			// Save built distance field volume to DDC
			FMemoryWriter Ar(DerivedData, /*bIsPersistent=*/ true);
			InTask->StaticMesh->GetRenderData()->LODResources[0].DistanceFieldData->Serialize(Ar, InTask->StaticMesh);
			GetDerivedDataCacheRef().Put(*InTask->DDCKey, DerivedData, InTask->StaticMesh->GetPathName());
			COOK_STAT(Timer.AddMiss(DerivedData.Num()));
		}

		if (!InTask->bGeneratedCardRepresentation)
		{
			BeginCacheMeshCardRepresentation(
				InTask->StaticMesh,
				InTask->StaticMesh->GetPlatformStaticMeshRenderData(InTask->StaticMesh, InTask->TargetPlatform),
				InTask->DDCKey,
				MoveTemp(InTask->SourceMeshData),
				MoveTemp(InTask->PartData));
		}
	}
}
#endif

FDistanceFieldAsyncQueue::FDistanceFieldAsyncQueue() 
	: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	FAssetCompilingManager::Get().RegisterManager(this);
	FAssetCompilingManager::Get().OnAssetPostCompileEvent().AddRaw(this, &FDistanceFieldAsyncQueue::OnAssetPostCompile);

#if WITH_EDITOR
	MeshUtilities = NULL;

	const int32 MaxConcurrency = -1;
	// In Editor, we allow faster compilation by letting the asset compiler's scheduler organize work.
	FQueuedThreadPool* InnerThreadPool = FAssetCompilingManager::Get().GetThreadPool();
#else
	const int32 MaxConcurrency = 1;
	FQueuedThreadPool* InnerThreadPool = GThreadPool;
#endif

	if (InnerThreadPool != nullptr)
	{
		ThreadPool = MakeUnique<FQueuedThreadPoolWrapper>(InnerThreadPool, MaxConcurrency, [](EQueuedWorkPriority) { return EQueuedWorkPriority::Lowest; });
	}
	PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddRaw(this, &FDistanceFieldAsyncQueue::OnPostReachabilityAnalysis);
}

FDistanceFieldAsyncQueue::~FDistanceFieldAsyncQueue()
{
	FAssetCompilingManager::Get().OnAssetPostCompileEvent().RemoveAll(this);
	FAssetCompilingManager::Get().UnregisterManager(this);
	FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostReachabilityAnalysisHandle);
}

void FDistanceFieldAsyncQueue::OnPostReachabilityAnalysis()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::CancelUnreachableMeshes);

	CancelAndDeleteTaskByPredicate([this](FAsyncDistanceFieldTask* Task) { return IsTaskInvalid(Task); });
}

void FAsyncDistanceFieldTaskWorker::DoWork()
{
	GDistanceFieldAsyncQueue->Build(&Task);
}

FName FDistanceFieldAsyncQueue::GetStaticAssetTypeName()
{
	return TEXT("UE-MeshDistanceField");
}

FName FDistanceFieldAsyncQueue::GetAssetTypeName() const 
{ 
	return GetStaticAssetTypeName();
}

FTextFormat FDistanceFieldAsyncQueue::GetAssetNameFormat() const
{
	return LOCTEXT("MeshDistanceFieldNameFormat", "{0}|plural(one=Mesh Distance Field,other=Mesh Distance Fields)");
}

TArrayView<FName> FDistanceFieldAsyncQueue::GetDependentTypeNames() const 
{
#if WITH_EDITOR
	static FName DependentTypeNames[] = 
	{
		// DistanceField are a byproduct of StaticMesh so they have a natural dependencies toward them.
		FStaticMeshCompilingManager::GetStaticAssetTypeName() 
	};

	return TArrayView<FName>(DependentTypeNames);
#else
	return TArrayView<FName>();
#endif
}

int32 FDistanceFieldAsyncQueue::GetNumRemainingAssets() const 
{
	return GetNumOutstandingTasks(); 
}

void FDistanceFieldAsyncQueue::FinishAllCompilation() 
{
	BlockUntilAllBuildsComplete(); 
}

void FDistanceFieldAsyncQueue::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::FinishCompilationForObjects);

	TSet<UStaticMesh*> StaticMeshes;
	for (UObject* Object : InObjects)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			StaticMeshes.Add(StaticMesh);
		}
		else if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Object))
		{
			if (StaticMeshComponent->GetStaticMesh())
			{
				StaticMeshes.Add(StaticMeshComponent->GetStaticMesh());
			}
		}
	}

	for (UStaticMesh* StaticMesh : StaticMeshes)
	{
		BlockUntilBuildComplete(StaticMesh, false);
	}
}

void FDistanceFieldAsyncQueue::MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects)
{
	TSet<UStaticMesh*> StaticMeshes;
	for (UObject* Object : InObjects)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			StaticMeshes.Add(StaticMesh);
		}
	}

	if (StaticMeshes.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::MarkCompilationAsCanceled);

		CancelAndDeleteTaskByPredicate(
			[&StaticMeshes](FAsyncDistanceFieldTask* Task)
			{
				// Mark canceled is only a hint and should be non-blocking so only remove or cancel tasks that are either not started or already finished.
				if (Task->AsyncTask == nullptr || Task->AsyncTask->IsDone())
				{
					return StaticMeshes.Contains(Task->GenerateSource) || StaticMeshes.Contains(Task->StaticMesh);
				}

				return false;
			}
		);
	}
}

bool FDistanceFieldAsyncQueue::IsTaskInvalid(FAsyncDistanceFieldTask* Task) const
{
	return (Task->StaticMesh && Task->StaticMesh->IsUnreachable()) || (Task->GenerateSource && Task->GenerateSource->IsUnreachable());
}

void FDistanceFieldAsyncQueue::CancelAndDeleteTaskByPredicate(TFunctionRef<bool (FAsyncDistanceFieldTask*)> InShouldCancelPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::CancelAndDeleteTaskByPredicate);

	FScopeLock Lock(&CriticalSection);

	if (ReferencedTasks.Num() || PendingTasks.Num() || CompletedTasks.Num())
	{
		TSet<FAsyncDistanceFieldTask*> Removed;

		auto RemoveByPredicate =
			[&InShouldCancelPredicate, &Removed](TSet<FAsyncDistanceFieldTask*>& Tasks)
		{
			for (auto It = Tasks.CreateIterator(); It; ++It)
			{
				FAsyncDistanceFieldTask* Task = *It;
				if (InShouldCancelPredicate(Task))
				{
					Removed.Add(Task);
					It.RemoveCurrent();
				}
			}
		};

		RemoveByPredicate(PendingTasks);
		RemoveByPredicate(ReferencedTasks);
		RemoveByPredicate(CompletedTasks);

		Lock.Unlock();

		CancelAndDeleteTask(Removed);
	}
}

void FDistanceFieldAsyncQueue::CancelAndDeleteTask(const TSet<FAsyncDistanceFieldTask*>& Tasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::CancelAndDeleteTask);

	// Do all the cancellation first to make sure none of these tasks
	// get scheduled as we're waiting for completion.
	for (FAsyncDistanceFieldTask* Task : Tasks)
	{
		if (Task->AsyncTask)
		{
			Task->AsyncTask->Cancel();
		}
	}

	for (FAsyncDistanceFieldTask* Task : Tasks)
	{
		if (Task->AsyncTask)
		{
			Task->AsyncTask->EnsureCompletion();
			Task->AsyncTask.Reset();
		}
	}

	for (FAsyncDistanceFieldTask* Task : Tasks)
	{
		if (Task->GeneratedVolumeData != nullptr)
		{
			// Rendering thread may still be referencing the old one, use the deferred cleanup interface to delete it next frame when it is safe
			BeginCleanup(Task->GeneratedVolumeData);
		}

#if DO_GUARD_SLOW
		{
			FScopeLock Lock(&CriticalSection);
			check(!PendingTasks.Contains(Task));
			check(!ReferencedTasks.Contains(Task));
			check(!CompletedTasks.Contains(Task));
		}
#endif

		delete Task;
	}
}
 
void FDistanceFieldAsyncQueue::StartBackgroundTask(FAsyncDistanceFieldTask* Task)
{
	check(Task->AsyncTask == nullptr);
	Task->AsyncTask = MakeUnique<FAsyncTask<FAsyncDistanceFieldTaskWorker>>(*Task);

	int64 RequiredMemory = -1;

#if WITH_EDITOR
	if (CVarDistanceFieldsEstimate.GetValueOnAnyThread())
	{
		const FStaticMeshLODResources& LODModel = Task->GenerateSource->GetRenderData()->LODResources[0];

		FMeshDataForDerivedDataTask MeshData;
		MeshData.SourceMeshData = &Task->SourceMeshData;
		MeshData.LODModel = &LODModel;
		MeshData.SectionData = Task->SectionData;
		MeshData.Bounds = (FBoxSphereBounds3f)Task->GenerateSource->GetRenderData()->Bounds;


		RequiredMemory = MeshUtilities.load(std::memory_order_relaxed)->MemoryForGenerateSignedDistanceFieldVolumeData(
			MeshData,
			Task->DistanceFieldResolutionScale);
	}
#endif

	Task->AsyncTask->StartBackgroundTask(ThreadPool.Get(), EQueuedWorkPriority::Lowest, EQueuedWorkFlags::DoNotRunInsideBusyWait, RequiredMemory, TEXT("DistanceField") );
}

void FDistanceFieldAsyncQueue::ProcessPendingTasks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::ProcessPendingTasks);

	FScopeLock Lock(&CriticalSection);

	for (auto It = PendingTasks.CreateIterator(); It; ++It)
	{
		FAsyncDistanceFieldTask* Task = *It;
		if ((Task->GenerateSource == nullptr || !Task->GenerateSource->IsCompiling()) && (Task->StaticMesh == nullptr || !Task->StaticMesh->IsCompiling()))
		{
			StartBackgroundTask(Task);
			It.RemoveCurrent();
		}
	}
}

void FDistanceFieldAsyncQueue::ProcessPendingTasksForMesh(const UStaticMesh *InStaticMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::ProcessPendingTasksForMesh);

	FScopeLock Lock(&CriticalSection);

	for (auto It = PendingTasks.CreateIterator(); It; ++It)
	{
		FAsyncDistanceFieldTask* Task = *It;
		if ((Task->StaticMesh == InStaticMesh || Task->GenerateSource == InStaticMesh) &&
			(Task->GenerateSource == nullptr || !Task->GenerateSource->IsCompiling()) && 
			(Task->StaticMesh == nullptr || !Task->StaticMesh->IsCompiling()))
		{
			StartBackgroundTask(Task);
			It.RemoveCurrent();
		}
	}
}

void FDistanceFieldAsyncQueue::AddTask(FAsyncDistanceFieldTask* Task)
{
#if WITH_EDITOR
	// This could happen during the cancellation of async static mesh build
	// Simply delete the task if the static mesh are being garbage collected
	if (IsTaskInvalid(Task))
	{
		CancelAndDeleteTask({Task});
		return;
	}

	// This is ok to only use relaxed atomics here since if threads are racing they should all get the same module pointer to store.
	if (!MeshUtilities.load(std::memory_order_relaxed))
	{
		MeshUtilities.store(&FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities")), std::memory_order_relaxed);
	}
	
	const bool bUseAsyncBuild = GUseAsyncDistanceFieldBuildQueue || !IsInGameThread();
	const bool bIsCompiling = (Task->GenerateSource && Task->GenerateSource->IsCompiling()) || (Task->StaticMesh && Task->StaticMesh->IsCompiling());

	{
		// Array protection when called from multiple threads
		FScopeLock Lock(&CriticalSection);
		check(!CompletedTasks.Contains(Task)); // reusing same pointer for a new task that is marked completed but has been canceled...
		ReferencedTasks.Add(Task);

		// The Source Mesh's RenderData is not ready yet, postpone the build
		if (bIsCompiling)
		{
			PendingTasks.Add(Task);
		}
		else if (bUseAsyncBuild)
		{
			// Make sure the Task is launched while we hold the lock to avoid
			// race with cancellation
			StartBackgroundTask(Task);
		}
	}
	
	if (!bIsCompiling && !bUseAsyncBuild)
	{
		Build(Task);
	}
#else
	UE_LOGF(LogStaticMesh,Fatal,"Tried to build a distance field without editor support (this should have been done during cooking)");
#endif
}

void FDistanceFieldAsyncQueue::CancelBuild(UStaticMesh* InStaticMesh)
{
	CancelBuilds({ InStaticMesh });
}

void FDistanceFieldAsyncQueue::CancelBuilds(const TSet<UStaticMesh*>& InStaticMeshes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::CancelBuilds)

	CancelAndDeleteTaskByPredicate(
		[&InStaticMeshes](FAsyncDistanceFieldTask* Task)
		{
			return InStaticMeshes.Contains(Task->GenerateSource) || InStaticMeshes.Contains(Task->StaticMesh);
		}
	);
}

void FDistanceFieldAsyncQueue::CancelAllOutstandingBuilds()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::CancelAllOutstandingBuilds)

	TSet<FAsyncDistanceFieldTask*> OutstandingTasks;
	{
		FScopeLock Lock(&CriticalSection);
		PendingTasks.Reset();
		OutstandingTasks = MoveTemp(ReferencedTasks);
	}

	CancelAndDeleteTask(OutstandingTasks);
}

void FDistanceFieldAsyncQueue::RescheduleBackgroundTask(FAsyncDistanceFieldTask* InTask, EQueuedWorkPriority InPriority)
{
	if (InTask->AsyncTask && InTask->AsyncTask->GetPriority() != InPriority)
	{
		InTask->AsyncTask->Reschedule(GThreadPool, InPriority);
	}
}

int32 FDistanceFieldAsyncQueue::GetNumOutstandingTasks(const UStaticMesh* InStaticMesh) const
{
	FScopeLock Lock(&CriticalSection);

	// Pending tasks are already included in referenced tasks
	if (!ReferencedTasks.Num())
	{
		return 0;
	}

	int32 Count = 0;
	for (auto It = ReferencedTasks.CreateConstIterator(); It; ++It)
	{
		const FAsyncDistanceFieldTask* Task = *It;
		if (InStaticMesh == Task->GenerateSource || InStaticMesh == Task->StaticMesh)
		{
			++Count;
		}
	}

	return Count;
}

void FDistanceFieldAsyncQueue::BlockUntilBuildComplete(UStaticMesh* StaticMesh, bool bWarnIfBlocked)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::BlockUntilBuildComplete)

	// We will track the wait time here, but only the cycles used.
	// This function is called whether or not an async task is pending, 
	// so we have to look elsewhere to properly count how many resources have actually finished building.
	COOK_STAT(auto Timer = DistanceFieldCookStats::UsageStats.TimeAsyncWait());
	COOK_STAT(Timer.TrackCyclesOnly());
	bool bReferenced = false;
	bool bHadToBlock = false;
	double StartTime = 0;

	TSet<UStaticMesh*> RequiredFinishCompilation;
	do 
	{
		ProcessAsyncTasksForMesh(StaticMesh);

		bReferenced = false;
		RequiredFinishCompilation.Reset();

		// Reschedule the tasks we're waiting on as highest prio
		{
			FScopeLock Lock(&CriticalSection);
			for (FAsyncDistanceFieldTask* Task : ReferencedTasks)
			{
				if (Task->StaticMesh == StaticMesh ||
					Task->GenerateSource == StaticMesh)
				{
					bReferenced = true;

					// If the task we are waiting on depends on other static meshes
					// we need to force finish them too.
#if WITH_EDITOR
					
					if (Task->GenerateSource != nullptr &&
						Task->GenerateSource->IsCompiling())
					{
						RequiredFinishCompilation.Add(Task->GenerateSource);
					}

					if (Task->StaticMesh != nullptr &&
						Task->StaticMesh->IsCompiling())
					{
						RequiredFinishCompilation.Add(Task->StaticMesh);
					}
#endif

					RescheduleBackgroundTask(Task, EQueuedWorkPriority::Blocking);
				}
			}
		}

#if WITH_EDITOR
		// Call the finish compilation outside of the critical section since those compilations
		// might need to register new distance field tasks which also uses the critical section.
		if (RequiredFinishCompilation.Num())
		{
			FStaticMeshCompilingManager::Get().FinishCompilation(RequiredFinishCompilation.Array());
		}
#endif

		if (bReferenced)
		{
			if (!bHadToBlock)
			{
				StartTime = FPlatformTime::Seconds();
			}

			bHadToBlock = true;
			FPlatformProcess::Sleep(.01f);
		}
	} 
	while (bReferenced);

	if (bHadToBlock &&
		bWarnIfBlocked
#if WITH_EDITOR
		&& !FAutomationTestFramework::Get().GetCurrentTest() // HACK - Don't output this warning during automation test
#endif
		)
	{
		UE_LOGF(LogStaticMesh, Display, "Main thread blocked for %.3fs for async distance field build of %ls to complete!  This can happen if the mesh is rebuilt excessively.",
			(float)(FPlatformTime::Seconds() - StartTime), 
			*StaticMesh->GetName());
	}
}

void FDistanceFieldAsyncQueue::BlockUntilAllBuildsComplete()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::BlockUntilAllBuildsComplete)
	while (true)
	{
#if WITH_EDITOR
		FStaticMeshCompilingManager::Get().FinishAllCompilation();
#endif
		// Reschedule as highest prio since we're explicitly waiting on them
		{
			FScopeLock Lock(&CriticalSection);
			for (FAsyncDistanceFieldTask* Task : ReferencedTasks)
			{
				RescheduleBackgroundTask(Task, EQueuedWorkPriority::Blocking);
			}
		}

		ProcessAsyncTasks();

		if (GetNumOutstandingTasks() <= 0)
		{
			break;
		}

		FPlatformProcess::Sleep(.01f);
	} 
}

void FDistanceFieldAsyncQueue::Build(FAsyncDistanceFieldTask* Task)
{
#if WITH_EDITOR
	// Editor 'force delete' can null any UObject pointers which are seen by reference collecting (eg FProperty or serialized)
	if (Task->StaticMesh && Task->GenerateSource)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::Build);

		const FStaticMeshLODResources& LODModel = Task->GenerateSource->GetRenderData()->LODResources[0];

		FMeshDataForDerivedDataTask MeshData;
		MeshData.SourceMeshData = &Task->SourceMeshData;
		MeshData.LODModel = &LODModel;
		MeshData.SectionData = Task->SectionData;
		MeshData.Bounds = (FBoxSphereBounds3f)Task->GenerateSource->GetRenderData()->Bounds;
		MeshData.Parts = Task->PartData;
		if (!MeshData.Parts.IsEmpty())
		{
			MeshData.Nodes = Task->GenerateSource->GetNaniteSettings().NaniteAssemblyData.Nodes;
		}

		MeshUtilities.load(std::memory_order_relaxed)->GenerateSignedDistanceFieldVolumeData(
			Task->StaticMesh->GetName(),
			MeshData,
			Task->DistanceFieldResolutionScale,
			Task->bGenerateDistanceFieldAsIfTwoSided,
			*Task->GeneratedVolumeData);

		// Begin mesh card representation while we're async
		FAsyncCardRepresentationTask* CardTask = BeginCacheMeshCardRepresentationInternal(
			Task->StaticMesh,
			*Task->StaticMesh->GetRenderData(),
			Task->DDCKey,
			MoveTemp(Task->SourceMeshData),
			MoveTemp(Task->PartData),
			Task->GeneratedVolumeData);

		if (CardTask)
		{
			// If cooking, build the dependent mesh card representation data immediately in the existing worker task
			if (IsRunningCookCommandlet())
			{
				GCardRepresentationAsyncQueue->RunImmediateTask(CardTask);
			}
			else
			{
				GCardRepresentationAsyncQueue->AddTask(CardTask);
			}
		}

		Task->bGeneratedCardRepresentation = true;
	}

	{
		FScopeLock Lock(&CriticalSection);
		// Avoid adding to the completed list if the task has been canceled
		if (ReferencedTasks.Contains(Task))
		{
			CompletedTasks.Add(Task);
		}
	}
#endif
}

void FDistanceFieldAsyncQueue::ProcessAsyncTasks(bool bLimitExecutionTime)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::ProcessAsyncTasks);

	ProcessPendingTasks();

	FObjectCacheContextScope ObjectCacheScope; // Declared here for caching optimization purposes
	const double MaxProcessingTime = 0.016f;
	double StartTime = FPlatformTime::Seconds();
	bool bMadeProgress = false;
	while (!bLimitExecutionTime || (FPlatformTime::Seconds() - StartTime) < MaxProcessingTime)
	{
		FAsyncDistanceFieldTask* Task = nullptr;
		{
			FScopeLock Lock(&CriticalSection);
			if (CompletedTasks.Num() > 0)
			{
				// Pop any element from the set
				auto Iterator = CompletedTasks.CreateIterator();
				Task = *Iterator;
				Iterator.RemoveCurrent();

				verify(ReferencedTasks.Remove(Task));
			}
		}

		if (Task == nullptr)
		{
			break;
		}
		bMadeProgress = true;

		FinalizeAsyncTask(Task, ObjectCacheScope);

		delete Task;
	}

	if (bMadeProgress)
	{
		Notification->Update(GetNumRemainingAssets());
	}
#endif
}

void FDistanceFieldAsyncQueue::ProcessAsyncTasksForMesh(const UStaticMesh* InStaticMesh, bool bLimitExecutionTime)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldAsyncQueue::ProcessAsyncTasksForMesh);

	ProcessPendingTasksForMesh(InStaticMesh);

	FObjectCacheContextScope ObjectCacheScope;  // Declared here for mem/caching optimization purposes
	const double MaxProcessingTime = 0.016f;
	double StartTime = FPlatformTime::Seconds();
	bool bMadeProgress = false;
	while (!bLimitExecutionTime || (FPlatformTime::Seconds() - StartTime) < MaxProcessingTime)
	{
		FAsyncDistanceFieldTask* Task = nullptr;
		{
			// Try and find a task that matches the static mesh
			FScopeLock Lock(&CriticalSection);
			for (auto CompletedTaskIt = CompletedTasks.CreateIterator(); CompletedTaskIt; ++CompletedTaskIt)
			{
				FAsyncDistanceFieldTask* TaskToCheck = *CompletedTaskIt;
				
				if (TaskToCheck != nullptr && (TaskToCheck->StaticMesh == InStaticMesh || TaskToCheck->GenerateSource == InStaticMesh))
				{
					Task = TaskToCheck;
					CompletedTaskIt.RemoveCurrent();
					verify(ReferencedTasks.Remove(Task));
					break;
				}
			}
		}

		if (Task == nullptr)
		{
			break;
		}
		bMadeProgress = true;

		FinalizeAsyncTask(Task, ObjectCacheScope);

		delete Task;
	}

	if (bMadeProgress)
	{
		Notification->Update(GetNumRemainingAssets());
	}
#endif
}

void FDistanceFieldAsyncQueue::Shutdown()
{
	CancelAllOutstandingBuilds();

	UE_LOGF(LogStaticMesh, Log, "Abandoning remaining async distance field tasks for shutdown");
	ThreadPool.Reset();
}

FLandscapeTextureAtlas::FLandscapeTextureAtlas(ESubAllocType InSubAllocType)
	: MaxDownSampleLevel(0)
	, Generation(0)
	, SubAllocType(InSubAllocType)
{}

void FLandscapeTextureAtlas::InitializeIfNeeded()
{
	const bool bHeight = SubAllocType == SAT_Height;
	const uint32 LocalTileSize = bHeight ? GHeightFieldAtlasTileSize : GHFVisibilityAtlasTileSize;
	const uint32 LocalDimInTiles = bHeight ? GHeightFieldAtlasDimInTiles : GHFVisibilityAtlasDimInTiles;
	const uint32 LocalDownSampleLevel = bHeight ? GHeightFieldAtlasDownSampleLevel : GHFVisibilityAtlasDownSampleLevel;

	if (!AtlasTextureRHI
		|| AddrSpaceAllocator.TileSize != LocalTileSize
		|| AddrSpaceAllocator.DimInTiles != LocalDimInTiles
		|| MaxDownSampleLevel != LocalDownSampleLevel)
	{
		AddrSpaceAllocator.Init(LocalTileSize, 1, LocalDimInTiles);

		for (int32 Idx = 0; Idx < PendingStreamingTextures.Num(); ++Idx)
		{
			UTexture2D* Texture = PendingStreamingTextures[Idx];
			Texture->bForceMiplevelsToBeResident = false;
		}
		PendingStreamingTextures.Empty();

		for (TSet<FAllocation>::TConstIterator It(CurrentAllocations); It; ++It)
		{
			check(!PendingAllocations.Contains(*It));
			PendingAllocations.Add(*It);
		}

		CurrentAllocations.Reset();

		const uint32 SizeX = AddrSpaceAllocator.DimInTexels;
		const uint32 SizeY = AddrSpaceAllocator.DimInTexels;
		const ETextureCreateFlags Flags = TexCreate_ShaderResource | TexCreate_UAV;
		const EPixelFormat Format = bHeight ? PF_R8G8 : PF_G8;
		const TCHAR* Name = bHeight ? TEXT("HeightFieldAtlas") : TEXT("VisibilityAtlas");

		AtlasTextureRHI = AllocatePooledTexture(FRDGTextureDesc::Create2D(FIntPoint(SizeX, SizeY), Format, FClearValueBinding::None, Flags), Name);

		MaxDownSampleLevel = LocalDownSampleLevel;
		++Generation;
	}
}

void FLandscapeTextureAtlas::AddAllocation(UTexture2D* Texture, uint32 VisibilityChannel)
{
	check(Texture);

	FAllocation* Found = CurrentAllocations.Find(Texture);
	if (Found)
	{
		++Found->RefCount;
		return;
	}

	Found = FailedAllocations.Find(Texture);
	if (Found)
	{
		++Found->RefCount;
		return;
	}

	Found = PendingAllocations.Find(Texture);
	if (Found)
	{
		++Found->RefCount;
	}
	else
	{
		PendingAllocations.Add(FAllocation(Texture, VisibilityChannel));
	}
}

void FLandscapeTextureAtlas::RemoveAllocation(UTexture2D* Texture)
{
	FSetElementId Id = PendingAllocations.FindId(Texture);
	if (Id.IsValidId())
	{
		check(PendingAllocations[Id].RefCount);
		if (!--PendingAllocations[Id].RefCount)
		{
			check(!PendingStreamingTextures.Contains(Texture));
			PendingAllocations.Remove(Id);
		}
		return;
	}

	Id = FailedAllocations.FindId(Texture);
	if (Id.IsValidId())
	{
		check(FailedAllocations[Id].RefCount);
		if (!--FailedAllocations[Id].RefCount)
		{
			check(!PendingStreamingTextures.Contains(Texture));
			FailedAllocations.Remove(Id);
		}
		return;
	}

	Id = CurrentAllocations.FindId(Texture);
	if (Id.IsValidId())
	{
		FAllocation& Allocation = CurrentAllocations[Id];
		check(Allocation.RefCount && Allocation.Handle != INDEX_NONE);
		if (!--Allocation.RefCount)
		{
			AddrSpaceAllocator.Free(Allocation.Handle);
			PendingStreamingTextures.Remove(Texture);
			CurrentAllocations.Remove(Id);
		}
	}
}

class FUploadLandscapeTextureToAtlasCS : public FGlobalShader
{
public:
	BEGIN_SHADER_PARAMETER_STRUCT(FSharedParameters, )
		SHADER_PARAMETER(FUintVector4, UpdateRegionOffsetAndSize)
		SHADER_PARAMETER(FVector4f, SourceScaleBias)
		SHADER_PARAMETER(uint32, SourceMipBias)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceTextureSampler)
	END_SHADER_PARAMETER_STRUCT()

	FUploadLandscapeTextureToAtlasCS() = default;

	FUploadLandscapeTextureToAtlasCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);
	}

	static constexpr uint32 ThreadGroupSizeX = 8;
	static constexpr uint32 ThreadGroupSizeY = 8;
};

class FUploadHeightFieldToAtlasCS : public FUploadLandscapeTextureToAtlasCS
{
	DECLARE_GLOBAL_SHADER(FUploadHeightFieldToAtlasCS);
	SHADER_USE_PARAMETER_STRUCT(FUploadHeightFieldToAtlasCS, FUploadLandscapeTextureToAtlasCS);

	using FPermutationDomain = FShaderPermutationNone;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER_STRUCT_INCLUDE(FUploadLandscapeTextureToAtlasCS::FSharedParameters, SharedParams)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float2>, RWHeightFieldAtlas)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FUploadHeightFieldToAtlasCS, "/Engine/Private/HeightFieldAtlasManagement.usf", "UploadHeightFieldToAtlasCS", SF_Compute);

class FUploadVisibilityToAtlasCS : public FUploadLandscapeTextureToAtlasCS
{
	DECLARE_GLOBAL_SHADER(FUploadVisibilityToAtlasCS);
	SHADER_USE_PARAMETER_STRUCT(FUploadVisibilityToAtlasCS, FUploadLandscapeTextureToAtlasCS);

	using FPermutationDomain = FShaderPermutationNone;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FUploadLandscapeTextureToAtlasCS::FSharedParameters, SharedParams)
		SHADER_PARAMETER(FVector4f, VisibilityChannelMask)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWVisibilityAtlas)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FUploadVisibilityToAtlasCS, "/Engine/Private/HeightFieldAtlasManagement.usf", "UploadVisibilityToAtlasCS", SF_Compute);

uint32 FLandscapeTextureAtlas::CalculateDownSampleLevel(uint32 SizeX, uint32 SizeY) const
{
	const uint32 TileSize = AddrSpaceAllocator.TileSize;

	for (uint32 CurLevel = 0; CurLevel <= MaxDownSampleLevel; ++CurLevel)
	{
		const uint32 DownSampledSizeX = SizeX >> CurLevel;
		const uint32 DownSampledSizeY = SizeY >> CurLevel;

		if (DownSampledSizeX <= TileSize && DownSampledSizeY <= TileSize)
		{
			return CurLevel;
		}
	}

	return MaxDownSampleLevel;
}

void FLandscapeTextureAtlas::UpdateAllocations(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type InFeatureLevel)
{
	// Mask should be set in FSceneRenderer::PrepareDistanceFieldScene before calling this
	check(GraphBuilder.RHICmdList.GetGPUMask() == FRHIGPUMask::All());

	InitializeIfNeeded();

	TArray<FPendingUpload, TInlineAllocator<8>> PendingUploads;

	auto AllocSortPred = [](const FAllocation& A, const FAllocation& B)
	{
		const int32 SizeA = FMath::Max(A.SourceTexture->GetSizeX(), A.SourceTexture->GetSizeY());
		const int32 SizeB = FMath::Max(B.SourceTexture->GetSizeX(), B.SourceTexture->GetSizeY());
		return SizeA < SizeB;
	};

	for (int32 Idx = 0; Idx < PendingStreamingTextures.Num(); ++Idx)
	{
		UTexture2D* SourceTexture = PendingStreamingTextures[Idx];
		const uint32 SizeX = SourceTexture->GetSizeX();
		const uint32 SizeY = SourceTexture->GetSizeY();
		const uint32 DownSampleLevel = CalculateDownSampleLevel(SizeX, SizeY);
		const uint32 NumMissingMips = SourceTexture->GetNumMips() - SourceTexture->GetNumResidentMips();

		if (NumMissingMips <= DownSampleLevel)
		{
			SourceTexture->bForceMiplevelsToBeResident = false;
			const uint32 SourceMipBias = DownSampleLevel - NumMissingMips;
			const FAllocation* Allocation = CurrentAllocations.Find(SourceTexture);
			check(Allocation && Allocation->Handle != INDEX_NONE);
			const uint32 VisibilityChannel = Allocation->VisibilityChannel;
			PendingUploads.Emplace(SourceTexture, SizeX >> DownSampleLevel, SizeY >> DownSampleLevel, SourceMipBias, Allocation->Handle, VisibilityChannel);
			PendingStreamingTextures.RemoveAtSwap(Idx--);
		}
	}

	if (PendingAllocations.Num())
	{
		PendingAllocations.Sort(AllocSortPred);
		bool bAllocFailed = false;

		for (TSet<FAllocation>::TConstIterator It(PendingAllocations); It; ++It)
		{
			FAllocation TmpAllocation = *It;

			if (!bAllocFailed)
			{
				UTexture2D* SourceTexture = TmpAllocation.SourceTexture;
				const uint32 SizeX = SourceTexture->GetSizeX();
				const uint32 SizeY = SourceTexture->GetSizeY();
				const uint32 DownSampleLevel = CalculateDownSampleLevel(SizeX, SizeY);
				const uint32 DownSampledSizeX = SizeX >> DownSampleLevel;
				const uint32 DownSampledSizeY = SizeY >> DownSampleLevel;
				const uint32 Handle = AddrSpaceAllocator.Alloc(DownSampledSizeX, DownSampledSizeY);
				const uint32 VisibilityChannel = TmpAllocation.VisibilityChannel;

				if (Handle == INDEX_NONE)
				{
					FailedAllocations.Add(TmpAllocation);
					bAllocFailed = true;
					continue;
				}

				const uint32 NumMissingMips = SourceTexture->GetNumMips() - SourceTexture->GetNumResidentMips();
				const uint32 SourceMipBias = NumMissingMips > DownSampleLevel ? 0 : DownSampleLevel - NumMissingMips;

				if (NumMissingMips > DownSampleLevel)
				{
					SourceTexture->bForceMiplevelsToBeResident = true;
					check(!PendingStreamingTextures.Contains(SourceTexture));
					PendingStreamingTextures.Add(SourceTexture);
				}

				TmpAllocation.Handle = Handle;
				CurrentAllocations.Add(TmpAllocation);
				PendingUploads.Emplace(SourceTexture, DownSampledSizeX, DownSampledSizeY, SourceMipBias, Handle, VisibilityChannel);
			}
			else
			{
				FailedAllocations.Add(TmpAllocation);
			}
		}

		if (bAllocFailed)
		{
			FailedAllocations.Sort(AllocSortPred);
		}
		PendingAllocations.Empty();
	}

	if (FailedAllocations.Num())
	{
		for (TSet<FAllocation>::TIterator It(FailedAllocations); It; ++It)
		{
			FAllocation TmpAllocation = *It;
			UTexture2D* SourceTexture = TmpAllocation.SourceTexture;
			const uint32 SizeX = SourceTexture->GetSizeX();
			const uint32 SizeY = SourceTexture->GetSizeY();
			const uint32 DownSampleLevel = CalculateDownSampleLevel(SizeX, SizeY);
			const uint32 DownSampledSizeX = SizeX >> DownSampleLevel;
			const uint32 DownSampledSizeY = SizeY >> DownSampleLevel;
			const uint32 Handle = AddrSpaceAllocator.Alloc(DownSampledSizeX, DownSampledSizeY);
			const uint32 VisibilityChannel = TmpAllocation.VisibilityChannel;

			if (Handle == INDEX_NONE)
			{
				break;
			}

			const uint32 NumMissingMips = SourceTexture->GetNumMips() - SourceTexture->GetNumResidentMips();
			const uint32 SourceMipBias = NumMissingMips > DownSampleLevel ? 0 : DownSampleLevel - NumMissingMips;

			if (NumMissingMips > DownSampleLevel)
			{
				SourceTexture->bForceMiplevelsToBeResident = true;
				check(!PendingStreamingTextures.Contains(SourceTexture));
				PendingStreamingTextures.Add(SourceTexture);
			}

			TmpAllocation.Handle = Handle;
			CurrentAllocations.Add(TmpAllocation);
			PendingUploads.Emplace(SourceTexture, DownSampledSizeX, DownSampledSizeY, SourceMipBias, Handle, VisibilityChannel);
			It.RemoveCurrent();
		}
	}

	if (PendingUploads.Num())
	{
		FRDGTextureUAV* AtlasTextureUAV = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(AtlasTextureRHI), ERDGUnorderedAccessViewFlags::SkipBarrier);

		if (SubAllocType == SAT_Height)
		{
			TShaderMapRef<FUploadHeightFieldToAtlasCS> ComputeShader(GetGlobalShaderMap(InFeatureLevel));
			for (int32 Idx = 0; Idx < PendingUploads.Num(); ++Idx)
			{
				typename FUploadHeightFieldToAtlasCS::FParameters* Parameters = GraphBuilder.AllocParameters<typename FUploadHeightFieldToAtlasCS::FParameters>();
				const FIntPoint UpdateRegion = PendingUploads[Idx].SetShaderParameters(Parameters, *this, AtlasTextureUAV);

				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("UploadHeightFieldToAtlas"), ComputeShader, Parameters, FIntVector(UpdateRegion.X, UpdateRegion.Y, 1));
			}
		}
		else
		{
			TShaderMapRef<FUploadVisibilityToAtlasCS> ComputeShader(GetGlobalShaderMap(InFeatureLevel));
			for (int32 Idx = 0; Idx < PendingUploads.Num(); ++Idx)
			{
				typename FUploadVisibilityToAtlasCS::FParameters* Parameters = GraphBuilder.AllocParameters<typename FUploadVisibilityToAtlasCS::FParameters>();
				const FIntPoint UpdateRegion = PendingUploads[Idx].SetShaderParameters(Parameters, *this, AtlasTextureUAV);

				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("UploadVisibilityToAtlas"), ComputeShader, Parameters, FIntVector(UpdateRegion.X, UpdateRegion.Y, 1));
			}
		}
	}
}

void FLandscapeTextureAtlas::UpdateAllocations(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type InFeatureLevel)
{
	FRDGBuilder GraphBuilder(RHICmdList);
	UpdateAllocations(GraphBuilder, InFeatureLevel);
	GraphBuilder.Execute();
}

uint32 FLandscapeTextureAtlas::GetAllocationHandle(UTexture2D* Texture) const
{
	const FAllocation* Allocation = CurrentAllocations.Find(Texture);
	return Allocation ? Allocation->Handle : INDEX_NONE;
}

FVector4f FLandscapeTextureAtlas::GetAllocationScaleBias(uint32 Handle) const
{
	return AddrSpaceAllocator.GetScaleBias(Handle);
}

FRDGTexture* FLandscapeTextureAtlas::GetAtlasTexture(FRDGBuilder& GraphBuilder) const
{
	return TryRegisterExternalTexture(GraphBuilder, AtlasTextureRHI);
}

void FLandscapeTextureAtlas::FSubAllocator::Init(uint32 InTileSize, uint32 InBorderSize, uint32 InDimInTiles)
{
	check(InDimInTiles && !(InDimInTiles & (InDimInTiles - 1)));

	TileSize = InTileSize;
	BorderSize = InBorderSize;
	TileSizeWithBorder = InTileSize + 2 * InBorderSize;
	DimInTiles = InDimInTiles;
	DimInTilesShift = FMath::CountBits(InDimInTiles - 1);
	DimInTilesMask = InDimInTiles - 1;
	DimInTexels = InDimInTiles * TileSizeWithBorder;
	MaxNumTiles = InDimInTiles * InDimInTiles;
	
	TexelSize = 1.f / DimInTexels;
	TileScale = TileSize * TexelSize;

	LevelOffsets.Empty();
	MarkerQuadTree.Empty();
	SubAllocInfos.Empty();

	uint32 NumBits = 0;
	for (uint32 Level = 1; Level <= DimInTiles; Level <<= 1)
	{
		const uint32 NumQuadsInLevel = Level * Level;
		LevelOffsets.Add(NumBits);
		NumBits += NumQuadsInLevel;
	}
	MarkerQuadTree.Add(false, NumBits);
}

uint32 FLandscapeTextureAtlas::FSubAllocator::Alloc(uint32 SizeX, uint32 SizeY)
{
	const uint32 NumTiles1D = FMath::DivideAndRoundUp(FMath::Max(SizeX, SizeY), TileSize);
	check(NumTiles1D <= DimInTiles);
	const uint32 NumLevels = LevelOffsets.Num();
	const uint32 Level = NumLevels - FMath::CeilLogTwo(NumTiles1D) - 1;
	const uint32 LevelOffset = LevelOffsets[Level];
	const uint32 QuadsInLevel1D = 1u << Level;
	const uint32 SearchEnd = LevelOffset + QuadsInLevel1D * QuadsInLevel1D;

	uint32 QuadIdx = LevelOffset;
	for (; QuadIdx < SearchEnd; ++QuadIdx)
	{
		if (!MarkerQuadTree[QuadIdx])
		{
			break;
		}
	}

	if (QuadIdx != SearchEnd)
	{
		const uint32 QuadIdxInLevel = QuadIdx - LevelOffset;
		
		uint32 ParentLevel = Level;
		uint32 ParentQuadIdxInLevel = QuadIdxInLevel;
		for (; ParentLevel != (uint32)-1; --ParentLevel)
		{
			const uint32 ParentLevelOffset = LevelOffsets[ParentLevel];
			const uint32 ParentQuadIdx = ParentLevelOffset + ParentQuadIdxInLevel;
			FBitReference Marker = MarkerQuadTree[ParentQuadIdx];
			if (Marker)
			{
				break;
			}
			Marker = true;
			ParentQuadIdxInLevel >>= 2;
		}

		uint32 ChildLevel = Level + 1;
		uint32 ChildQuadIdxInLevel = QuadIdxInLevel << 2;
		uint32 NumChildren = 4;
		for (; ChildLevel < NumLevels; ++ChildLevel)
		{
			const uint32 ChildQuadIdx = ChildQuadIdxInLevel + LevelOffsets[ChildLevel];
			for (uint32 Idx = 0; Idx < NumChildren; ++Idx)
			{
				check(!MarkerQuadTree[ChildQuadIdx + Idx]);
				MarkerQuadTree[ChildQuadIdx + Idx] = true;
			}
			ChildQuadIdxInLevel <<= 2;
			NumChildren <<= 2;
		}
		
		const uint32 QuadX = FMath::ReverseMortonCode2(QuadIdxInLevel);
		const uint32 QuadY = FMath::ReverseMortonCode2(QuadIdxInLevel >> 1);
		const uint32 QuadSizeInTiles1D = DimInTiles >> Level;
		const uint32 TileX = QuadX * QuadSizeInTiles1D;
		const uint32 TileY = QuadY * QuadSizeInTiles1D;

		FSubAllocInfo SubAllocInfo;
		SubAllocInfo.Level = Level;
		SubAllocInfo.QuadIdx = QuadIdx;
		SubAllocInfo.UVScaleBias.X = SizeX * TexelSize;
		SubAllocInfo.UVScaleBias.Y = SizeY * TexelSize;
		SubAllocInfo.UVScaleBias.Z = TileX / (float)DimInTiles + BorderSize * TexelSize;
		SubAllocInfo.UVScaleBias.W = TileY / (float)DimInTiles + BorderSize * TexelSize;

		return SubAllocInfos.Add(SubAllocInfo);
	}

	return INDEX_NONE;
}

void FLandscapeTextureAtlas::FSubAllocator::Free(uint32 Handle)
{
	check(SubAllocInfos.IsValidIndex(Handle));

	const FSubAllocInfo SubAllocInfo = SubAllocInfos[Handle];
	SubAllocInfos.RemoveAt(Handle);

	const uint32 Level = SubAllocInfo.Level;
	const uint32 QuadIdx = SubAllocInfo.QuadIdx;
	
	uint32 ChildLevel = Level;
	uint32 ChildIdxInLevel = QuadIdx - LevelOffsets[Level];
	uint32 NumChildren = 1;
	const uint32 NumLevels = LevelOffsets.Num();
	for (; ChildLevel < NumLevels; ++ChildLevel)
	{
		const uint32 ChildIdx = ChildIdxInLevel + LevelOffsets[ChildLevel];
		for (uint32 Idx = 0; Idx < NumChildren; ++Idx)
		{
			check(MarkerQuadTree[ChildIdx + Idx]);
			MarkerQuadTree[ChildIdx + Idx] = false;
		}
		ChildIdxInLevel <<= 2;
		NumChildren <<= 2;
	}

	uint32 TestIdxInLevel = (QuadIdx - LevelOffsets[Level]) & ~3u;
	uint32 ParentLevel = Level - 1;
	for (; ParentLevel != (uint32)-1; --ParentLevel)
	{
		const uint32 TestIdx = TestIdxInLevel + LevelOffsets[ParentLevel + 1];
		const bool bParentFree = !MarkerQuadTree[TestIdx] && !MarkerQuadTree[TestIdx + 1] && !MarkerQuadTree[TestIdx + 2] && !MarkerQuadTree[TestIdx + 3];
		if (!bParentFree)
		{
			break;
		}
		const uint32 ParentIdxInLevel = TestIdxInLevel >> 2;
		const uint32 ParentIdx = ParentIdxInLevel + LevelOffsets[ParentLevel];
		MarkerQuadTree[ParentIdx] = false;
		TestIdxInLevel = ParentIdxInLevel & ~3u;
	}
}

FVector4f FLandscapeTextureAtlas::FSubAllocator::GetScaleBias(uint32 Handle) const
{
	check(SubAllocInfos.IsValidIndex(Handle));
	return SubAllocInfos[Handle].UVScaleBias;
}

FIntPoint FLandscapeTextureAtlas::FSubAllocator::GetStartOffset(uint32 Handle) const
{
	check(SubAllocInfos.IsValidIndex(Handle));
	const FSubAllocInfo& Info = SubAllocInfos[Handle];
	const uint32 QuadIdxInLevel = Info.QuadIdx - LevelOffsets[Info.Level];
	const uint32 QuadX = FMath::ReverseMortonCode2(QuadIdxInLevel);
	const uint32 QuadY = FMath::ReverseMortonCode2(QuadIdxInLevel >> 1);
	const uint32 QuadSizeInTexels1D = (DimInTiles >> Info.Level) * TileSizeWithBorder;	
	return FIntPoint(QuadX * QuadSizeInTexels1D, QuadY * QuadSizeInTexels1D);
}

FLandscapeTextureAtlas::FAllocation::FAllocation()
	: SourceTexture(nullptr)
	, Handle(INDEX_NONE)
	, VisibilityChannel(0)
	, RefCount(0)
{}

FLandscapeTextureAtlas::FAllocation::FAllocation(UTexture2D* InTexture, uint32 InVisibilityChannel)
	: SourceTexture(InTexture)
	, Handle(INDEX_NONE)
	, VisibilityChannel(InVisibilityChannel)
	, RefCount(1)
{}

FLandscapeTextureAtlas::FPendingUpload::FPendingUpload(UTexture2D* Texture, uint32 SizeX, uint32 SizeY, uint32 MipBias, uint32 InHandle, uint32 Channel)
	: SourceTexture(Texture->GetResource()->TextureRHI)
	, SizesAndMipBias(FIntVector(SizeX, SizeY, MipBias))
	, VisibilityChannel(Channel)
	, Handle(InHandle)
{}

FIntPoint FLandscapeTextureAtlas::FPendingUpload::SetShaderParameters(void* ParamsPtr, const FLandscapeTextureAtlas& Atlas, FRDGTextureUAV* AtlasUAV) const
{
	if (Atlas.SubAllocType == SAT_Height)
	{
		typename FUploadHeightFieldToAtlasCS::FParameters* Params = (typename FUploadHeightFieldToAtlasCS::FParameters*)ParamsPtr;
		Params->RWHeightFieldAtlas = AtlasUAV;
		return SetCommonShaderParameters(&Params->SharedParams, Atlas);
	}
	else
	{
		typename FUploadVisibilityToAtlasCS::FParameters* Params = (typename FUploadVisibilityToAtlasCS::FParameters*)ParamsPtr;
		FVector4f ChannelMask(ForceInitToZero);
		ChannelMask[VisibilityChannel] = 1.f;
		Params->VisibilityChannelMask = ChannelMask;
		Params->RWVisibilityAtlas = AtlasUAV;
		return SetCommonShaderParameters(&Params->SharedParams, Atlas);
	}
}

FIntPoint FLandscapeTextureAtlas::FPendingUpload::SetCommonShaderParameters(void* ParamsPtr, const FLandscapeTextureAtlas& Atlas) const
{
	const uint32 DownSampledSizeX = SizesAndMipBias.X;
	const uint32 DownSampledSizeY = SizesAndMipBias.Y;
	const uint32 SourceMipBias = SizesAndMipBias.Z;
	const float InvDownSampledSizeX = 1.f / DownSampledSizeX;
	const float InvDownSampledSizeY = 1.f / DownSampledSizeY;
	const uint32 BorderSize = Atlas.AddrSpaceAllocator.BorderSize;
	const uint32 UpdateRegionSizeX = DownSampledSizeX + 2 * BorderSize;
	const uint32 UpdateRegionSizeY = DownSampledSizeY + 2 * BorderSize;
	const FIntPoint StartOffset = Atlas.AddrSpaceAllocator.GetStartOffset(Handle);

	typename FUploadLandscapeTextureToAtlasCS::FSharedParameters* CommonParams = (typename FUploadLandscapeTextureToAtlasCS::FSharedParameters*)ParamsPtr;
	CommonParams->UpdateRegionOffsetAndSize = FUintVector4(StartOffset.X, StartOffset.Y, UpdateRegionSizeX, UpdateRegionSizeY);
	CommonParams->SourceScaleBias = FVector4f(InvDownSampledSizeX, InvDownSampledSizeY, (.5f - BorderSize) * InvDownSampledSizeX, (.5f - BorderSize) * InvDownSampledSizeY);
	CommonParams->SourceMipBias = SourceMipBias;
	CommonParams->SourceTexture = SourceTexture;
	CommonParams->SourceTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	const uint32 NumGroupsX = FMath::DivideAndRoundUp(UpdateRegionSizeX, FUploadLandscapeTextureToAtlasCS::ThreadGroupSizeX);
	const uint32 NumGroupsY = FMath::DivideAndRoundUp(UpdateRegionSizeY, FUploadLandscapeTextureToAtlasCS::ThreadGroupSizeY);
	return FIntPoint(NumGroupsX, NumGroupsY);
}

#undef LOCTEXT_NAMESPACE
