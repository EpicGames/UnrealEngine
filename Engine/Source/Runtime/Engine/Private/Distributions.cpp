// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Distributions.cpp: Implementation of distribution classes.
=============================================================================*/

#include "Distributions.h"
#include "Math/InterpCurve.h"
#include "UObject/UnrealType.h"
#include "Distributions/Distribution.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/ParticleModule.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Distributions/DistributionVectorParameterBase.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Distributions/DistributionVectorUniformCurve.h"
#include "Sound/SoundNode.h"

#include "Distributions/DistributionFloatUniformCurve.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Distributions/DistributionFloatParameterBase.h"

// Moving UDistributions to PostInitProps to not be default sub-objects:
// Small enough value to be rounded to 0.0 in the editor 
// but significant enough to properly detect uninitialized defaults.
const float UDistribution::DefaultValue = 1.2345E-20f;

ENGINE_API uint32 GDistributionType = 1;

// The error threshold used when optimizing lookup table sample counts.
#define LOOKUP_TABLE_ERROR_THRESHOLD (0.05f)

// The maximum number of values to store in a lookup table.
#define LOOKUP_TABLE_MAX_SAMPLES (128)

// UDistribution will bake out (if CanBeBaked returns true)
#define DISTRIBUTIONS_BAKES_OUT 1

// The maximum number of samples must be a power of two.
static_assert((LOOKUP_TABLE_MAX_SAMPLES & (LOOKUP_TABLE_MAX_SAMPLES - 1)) == 0, "Lookup table max samples is not a power of two.");

// Log category for distributions.
DEFINE_LOG_CATEGORY_STATIC(LogDistributions,Warning,Warning);

UDistribution::UDistribution(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UDistribution::IsPostLoadThreadSafe() const
{
	return true;
}

/*-----------------------------------------------------------------------------
	Lookup table related functions.
------------------------------------------------------------------------------*/

//@todo.CONSOLE: Until we have cooking or something in place, these need to be exposed.
/**
 * Builds a lookup table that returns a constant value.
 * @param OutTable - The table to build.
 * @param ValuesPerEntry - The number of values per entry in the table.
 * @param Values - The values to place in the table.
 */
static void BuildConstantLookupTable( FDistributionLookupTable* OutTable, int32 ValuesPerEntry, const float* Values )
{
	check( OutTable != NULL );
	check( Values != NULL );

	OutTable->Values.Empty(ValuesPerEntry);
	OutTable->Values.AddUninitialized(ValuesPerEntry);
	OutTable->Op = RDO_None;
	OutTable->EntryCount = 1;
	OutTable->EntryStride = ValuesPerEntry;
	OutTable->SubEntryStride = 0;
	OutTable->TimeBias = 0.0f;
	OutTable->TimeScale = 0.0f;
	for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
	{
		OutTable->Values[ValueIndex] = Values[ValueIndex];
	}
}

/**
 * Builds a lookup table that returns zero.
 * @param OutTable - The table to build.
 * @param ValuesPerEntry - The number of values per entry in the table.
 */
static void BuildZeroLookupTable( FDistributionLookupTable* OutTable, int32 ValuesPerEntry )
{
	check( OutTable != NULL );
	check( ValuesPerEntry >= 1 && ValuesPerEntry <= 4 );

	float Zero[4] = {0};
	BuildConstantLookupTable( OutTable, ValuesPerEntry, Zero );
}

/**
 * Builds a lookup table from a distribution.
 * @param OutTable - The table to build.
 * @param Distribution - The distribution for which to build a lookup table.
 */
template <typename DistributionType>
void BuildLookupTable( FDistributionLookupTable* OutTable, const DistributionType& Distribution )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check(Distribution);

	// Always clear the table.
	OutTable->Empty();

	// Nothing to do if we don't have a distribution.
	if ( !Distribution->CanBeBaked() )
	{
		BuildZeroLookupTable( OutTable, Distribution->GetValueCount() );
		return;
	}

	// Always build a lookup table of maximal size. This can/will be optimized later.
	const int32 EntryCount = LOOKUP_TABLE_MAX_SAMPLES;

	// Determine the domain of the distribution.
	float MinIn, MaxIn;
	Distribution->GetInRange( MinIn, MaxIn );
	const float TimeScale = (MaxIn - MinIn) / (float)(EntryCount - 1);

	// Get the operation to use, and calculate the number of values needed for that operation.
	const uint8 Op = Distribution->GetOperation();
	const int32 ValuesPerEntry = Distribution->GetValueCount();
	const uint32 EntryStride = ((Op == RDO_None) ? 1 : 2) * (uint32)ValuesPerEntry;

	// Get the lock flag to use.
	const uint8 LockFlag = Distribution->GetLockFlag();

	// Allocate a lookup table of the appropriate size.
	OutTable->Op = Op;
	OutTable->EntryCount = EntryCount;
	OutTable->EntryStride = EntryStride;
	OutTable->SubEntryStride = (Op == RDO_None) ? 0 : ValuesPerEntry;
	OutTable->TimeScale = (TimeScale > 0.0f) ? (1.0f / TimeScale) : 0.0f;
	OutTable->TimeBias = MinIn;
	OutTable->Values.Empty( EntryCount * EntryStride );
	OutTable->Values.AddZeroed( EntryCount * EntryStride );
	OutTable->LockFlag = LockFlag;

	// Sample the distribution.
	for ( uint32 SampleIndex = 0; SampleIndex < EntryCount; SampleIndex++ )
	{
		const float Time = MinIn + SampleIndex * TimeScale;
		float Values[8];
		Distribution->InitializeRawEntry( Time, Values );
		for ( uint32 ValueIndex = 0; ValueIndex < EntryStride; ValueIndex++ )
		{
			OutTable->Values[SampleIndex * EntryStride + ValueIndex] = Values[ValueIndex];
		}
	}
}

/**
 * Appends one lookup table to another.
 * @param Table - Table which contains the first set of components [1-3].
 * @param OtherTable - Table to append which contains a single component.
 */
static void AppendLookupTable( FDistributionLookupTable* Table, const FDistributionLookupTable& OtherTable )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check( Table != NULL );
	check( Table->GetValuesPerEntry() >= 1 && Table->GetValuesPerEntry() <= 3 );
	check( OtherTable.GetValuesPerEntry() == 1 );

	// Copy the input table.
	FDistributionLookupTable TableCopy = *Table;

	// Compute the domain of the composed distribution.
	const float OneOverTimeScale = (TableCopy.TimeScale == 0.0f) ? 0.0f : 1.0f / TableCopy.TimeScale;
	const float OneOverOtherTimeScale = (OtherTable.TimeScale == 0.0f) ? 0.0f : 1.0f / OtherTable.TimeScale;
	const float MinIn = FMath::Min( TableCopy.TimeBias, OtherTable.TimeBias );
	const float MaxIn = FMath::Max( TableCopy.TimeBias + (TableCopy.EntryCount-1) * OneOverTimeScale, OtherTable.TimeBias + (OtherTable.EntryCount-1) * OneOverOtherTimeScale );

	const int32 InValuesPerEntry = TableCopy.GetValuesPerEntry();
	const int32 OtherValuesPerEntry = 1;
	const int32 NewValuesPerEntry = InValuesPerEntry + OtherValuesPerEntry;
	const uint8 NewOp = (TableCopy.Op == RDO_None) ? OtherTable.Op : TableCopy.Op;
	const int32 NewEntryCount = LOOKUP_TABLE_MAX_SAMPLES;
	const int32 NewStride = (NewOp == RDO_None) ? NewValuesPerEntry : NewValuesPerEntry * 2;
	const float NewTimeScale = (MaxIn - MinIn) / (float)(NewEntryCount - 1);

	// Now build the new lookup table.
	Table->Op = NewOp;
	Table->EntryCount = NewEntryCount;
	Table->EntryStride = NewStride;
	Table->SubEntryStride = (NewOp == RDO_None) ? 0 : NewValuesPerEntry;
	Table->TimeScale = (NewTimeScale > 0.0f) ? 1.0f / NewTimeScale : 0.0f;
	Table->TimeBias = MinIn;
	Table->Values.Empty( NewEntryCount * NewStride );
	Table->Values.AddZeroed( NewEntryCount * NewStride );
	for ( int32 SampleIndex = 0; SampleIndex < NewEntryCount; ++SampleIndex )
	{
		const float* InEntry1;
		const float* InEntry2;
		const float* OtherEntry1;
		const float* OtherEntry2;
		float InLerpAlpha;
		float OtherLerpAlpha;

		const float Time = MinIn + SampleIndex * NewTimeScale;
		TableCopy.GetEntry( Time, InEntry1, InEntry2, InLerpAlpha );
		OtherTable.GetEntry( Time, OtherEntry1, OtherEntry2, OtherLerpAlpha );

		// Store sub-entry 1.
		for ( int32 ValueIndex = 0; ValueIndex < InValuesPerEntry; ++ValueIndex )
		{
			Table->Values[SampleIndex * NewStride + ValueIndex] =
				FMath::Lerp( InEntry1[ValueIndex], InEntry2[ValueIndex], InLerpAlpha );
		}
		Table->Values[SampleIndex * NewStride + InValuesPerEntry] =
			FMath::Lerp( OtherEntry1[0], OtherEntry2[0], OtherLerpAlpha );

		// Store sub-entry 2 if needed. 
		if ( NewOp != RDO_None )
		{
			InEntry1 += TableCopy.SubEntryStride;
			InEntry2 += TableCopy.SubEntryStride;
			OtherEntry1 += OtherTable.SubEntryStride;
			OtherEntry2 += OtherTable.SubEntryStride;

			for ( int32 ValueIndex = 0; ValueIndex < InValuesPerEntry; ++ValueIndex )
			{
				Table->Values[SampleIndex * NewStride + NewValuesPerEntry + ValueIndex] =
					FMath::Lerp( InEntry1[ValueIndex], InEntry2[ValueIndex], InLerpAlpha );
			}
			Table->Values[SampleIndex * NewStride + NewValuesPerEntry + InValuesPerEntry] =
				FMath::Lerp( OtherEntry1[0], OtherEntry2[0], OtherLerpAlpha );
		}
	}
}

/**
 * Keeps only the first components of each entry in the table.
 * @param Table - Table to slice.
 * @param ChannelsToKeep - The number of channels to keep.
 */
static void SliceLookupTable( FDistributionLookupTable* Table, int32 ChannelsToKeep )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check( Table != NULL );
	check( Table->GetValuesPerEntry() >= ChannelsToKeep );

	// If the table only has the requested number of channels there is nothing to do.
	if ( Table->GetValuesPerEntry() == ChannelsToKeep )
	{
		return;
	}

	// Copy the table.
	FDistributionLookupTable OldTable = *Table;

	// The new table will have the same number of entries as the input table.
	const int32 NewEntryCount = OldTable.EntryCount;
	const int32 NewStride = (OldTable.Op == RDO_None) ? (ChannelsToKeep) : (2*ChannelsToKeep);
	Table->Op = OldTable.Op;
	Table->EntryCount = NewEntryCount;
	Table->EntryStride = NewStride;
	Table->SubEntryStride = (OldTable.Op == RDO_None) ? (0) : (ChannelsToKeep);
	Table->TimeBias = OldTable.TimeBias;
	Table->TimeScale = OldTable.TimeScale;
	Table->Values.Empty( NewEntryCount * NewStride );
	Table->Values.AddZeroed( NewEntryCount * NewStride );

	// Copy values over.
	for ( int32 EntryIndex = 0; EntryIndex < NewEntryCount; ++EntryIndex )
	{
		const float* RESTRICT SrcValues = &OldTable.Values[EntryIndex * OldTable.EntryStride];
		float* RESTRICT DestValues = &Table->Values[EntryIndex * Table->EntryStride];
		for ( int32 ValueIndex = 0; ValueIndex < ChannelsToKeep; ++ValueIndex )
		{
			DestValues[ValueIndex] = SrcValues[ValueIndex];
		}
		if ( OldTable.SubEntryStride > 0 )
		{
			SrcValues += OldTable.SubEntryStride;
			DestValues += Table->SubEntryStride;
			for ( int32 ValueIndex = 0; ValueIndex < ChannelsToKeep; ++ValueIndex )
			{
				DestValues[ValueIndex] = SrcValues[ValueIndex];
			}
		}
	}
}

/**
 * Scales each value in the lookup table by a constant.
 * @param InTable - Table to be scaled.
 * @param Scale - The amount to scale by.
 */
static void ScaleLookupTableByConstant( FDistributionLookupTable* Table, float Scale )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check( Table != NULL );

	for ( int32 ValueIndex = 0; ValueIndex < Table->Values.Num(); ++ValueIndex )
	{
		Table->Values[ValueIndex] *= Scale;
	}
}

/**
 * Scales each value in the lookup table by a constant.
 * @param InTable - Table to be scaled.
 * @param Scale - The amount to scale by.
 * @param ValueCount - The number of scale values.
 */
static void ScaleLookupTableByConstants( FDistributionLookupTable* Table, const float* Scale, int32 ValueCount )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check( Table != NULL );
	check( ValueCount == Table->GetValuesPerEntry() );

	const int32 EntryCount = Table->EntryCount;
	const int32 SubEntryCount = (Table->SubEntryStride > 0) ? 2 : 1;
	const int32 Stride = Table->EntryStride;
	const int32 SubEntryStride = Table->SubEntryStride;
	float* RESTRICT Values = Table->Values.GetData();

	for ( int32 Index = 0; Index < EntryCount; ++Index )
	{
		float* RESTRICT EntryValues = Values;
		for ( int32 SubEntryIndex = 0; SubEntryIndex < SubEntryCount; ++SubEntryIndex )
		{
			for ( int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex )
			{
				EntryValues[ValueIndex] *= Scale[ValueIndex];
			}
			EntryValues += SubEntryStride;
		}
		Values += Stride;
	}
}

/**
 * Adds a constant to each value in the lookup table.
 * @param InTable - Table to be scaled.
 * @param Addend - The amount to add by.
 * @param ValueCount - The number of values per entry.
 */
static void AddConstantToLookupTable( FDistributionLookupTable* Table, const float* Addend, int32 ValueCount )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check( Table != NULL );
	check( ValueCount == Table->GetValuesPerEntry() );

	const int32 EntryCount = Table->EntryCount;
	const int32 SubEntryCount = (Table->SubEntryStride > 0) ? 2 : 1;
	const int32 Stride = Table->EntryStride;
	const int32 SubEntryStride = Table->SubEntryStride;
	float* RESTRICT Values = Table->Values.GetData();

	for ( int32 Index = 0; Index < EntryCount; ++Index )
	{
		float* RESTRICT EntryValues = Values;
		for ( int32 SubEntryIndex = 0; SubEntryIndex < SubEntryCount; ++SubEntryIndex )
		{
			for ( int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex )
			{
				EntryValues[ValueIndex] += Addend[ValueIndex];
			}
			EntryValues += SubEntryStride;
		}
		Values += Stride;
	}
}

/**
 * Scales one lookup table by another.
 * @param Table - The table to scale.
 * @param OtherTable - The table to scale by. Must have one value per entry OR the same values per entry as Table.
 */
static void ScaleLookupTableByLookupTable( FDistributionLookupTable* Table, const FDistributionLookupTable& OtherTable )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check( Table != NULL );
	check( OtherTable.GetValuesPerEntry() == 1 || OtherTable.GetValuesPerEntry() == Table->GetValuesPerEntry() );

	// Copy the original table.
	FDistributionLookupTable InTable = *Table;

	// Compute the domain of the composed distribution.
	const float OneOverTimeScale = (InTable.TimeScale == 0.0f) ? 0.0f : 1.0f / InTable.TimeScale;
	const float OneOverOtherTimeScale = (OtherTable.TimeScale == 0.0f) ? 0.0f : 1.0f / OtherTable.TimeScale;
	const float MinIn = FMath::Min( InTable.TimeBias, OtherTable.TimeBias );
	const float MaxIn = FMath::Max( InTable.TimeBias + (InTable.EntryCount-1) * OneOverTimeScale, OtherTable.TimeBias + (OtherTable.EntryCount-1) * OneOverOtherTimeScale );

	const int32 ValuesPerEntry = InTable.GetValuesPerEntry();
	const int32 OtherValuesPerEntry = OtherTable.GetValuesPerEntry();
	const uint8 NewOp = (InTable.Op == RDO_None) ? OtherTable.Op : InTable.Op;
	const int32 NewEntryCount = LOOKUP_TABLE_MAX_SAMPLES;
	const int32 NewStride = (NewOp == RDO_None) ? ValuesPerEntry : ValuesPerEntry * 2;
	const float NewTimeScale = (MaxIn - MinIn) / (float)(NewEntryCount - 1);

	// Now build the new lookup table.
	Table->Op = NewOp;
	Table->EntryCount = NewEntryCount;
	Table->EntryStride = NewStride;
	Table->SubEntryStride = (NewOp == RDO_None) ? 0 : ValuesPerEntry;
	Table->TimeScale = (NewTimeScale > 0.0f) ? 1.0f / NewTimeScale : 0.0f;
	Table->TimeBias = MinIn;
	Table->Values.Empty( NewEntryCount * NewStride );
	Table->Values.AddZeroed( NewEntryCount * NewStride );
	for ( int32 SampleIndex = 0; SampleIndex < NewEntryCount; ++SampleIndex )
	{
		const float* InEntry1;
		const float* InEntry2;
		const float* OtherEntry1;
		const float* OtherEntry2;
		float InLerpAlpha;
		float OtherLerpAlpha;

		const float Time = MinIn + SampleIndex * NewTimeScale;
		InTable.GetEntry( Time, InEntry1, InEntry2, InLerpAlpha );
		OtherTable.GetEntry( Time, OtherEntry1, OtherEntry2, OtherLerpAlpha );

		// Scale sub-entry 1.
		for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
		{
			Table->Values[SampleIndex * NewStride + ValueIndex] =
				FMath::Lerp( InEntry1[ValueIndex], InEntry2[ValueIndex], InLerpAlpha ) *
				FMath::Lerp( OtherEntry1[ValueIndex % OtherValuesPerEntry], OtherEntry2[ValueIndex % OtherValuesPerEntry], OtherLerpAlpha );
		}

		// Scale sub-entry 2 if needed. 
		if ( NewOp != RDO_None )
		{
			InEntry1 += InTable.SubEntryStride;
			InEntry2 += InTable.SubEntryStride;
			OtherEntry1 += OtherTable.SubEntryStride;
			OtherEntry2 += OtherTable.SubEntryStride;

			for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
			{
				Table->Values[SampleIndex * NewStride + ValuesPerEntry + ValueIndex] =
					FMath::Lerp( InEntry1[ValueIndex], InEntry2[ValueIndex], InLerpAlpha ) *
					FMath::Lerp( OtherEntry1[ValueIndex % OtherValuesPerEntry], OtherEntry2[ValueIndex % OtherValuesPerEntry], OtherLerpAlpha );
			}
		}
	}
}

/**
 * Adds the values in one lookup table by another.
 * @param Table - The table to which to add values.
 * @param OtherTable - The table from which to add values. Must have one value per entry OR the same values per entry as Table.
 */
static void AddLookupTableToLookupTable( FDistributionLookupTable* Table, const FDistributionLookupTable& OtherTable )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check( Table != NULL );
	check( OtherTable.GetValuesPerEntry() == 1 || OtherTable.GetValuesPerEntry() == Table->GetValuesPerEntry() );

	// Copy the original table.
	FDistributionLookupTable InTable = *Table;

	// Compute the domain of the composed distribution.
	const float OneOverTimeScale = (InTable.TimeScale == 0.0f) ? 0.0f : 1.0f / InTable.TimeScale;
	const float OneOverOtherTimeScale = (OtherTable.TimeScale == 0.0f) ? 0.0f : 1.0f / OtherTable.TimeScale;
	const float MinIn = FMath::Min( InTable.TimeBias, OtherTable.TimeBias );
	const float MaxIn = FMath::Max( InTable.TimeBias + (InTable.EntryCount-1) * OneOverTimeScale, OtherTable.TimeBias + (OtherTable.EntryCount-1) * OneOverOtherTimeScale );

	const int32 ValuesPerEntry = InTable.GetValuesPerEntry();
	const int32 OtherValuesPerEntry = OtherTable.GetValuesPerEntry();
	const uint8 NewOp = (InTable.Op == RDO_None) ? OtherTable.Op : InTable.Op;
	const int32 NewEntryCount = LOOKUP_TABLE_MAX_SAMPLES;
	const int32 NewStride = (NewOp == RDO_None) ? ValuesPerEntry : ValuesPerEntry * 2;
	const float NewTimeScale = (MaxIn - MinIn) / (float)(NewEntryCount - 1);

	// Now build the new lookup table.
	Table->Op = NewOp;
	Table->EntryCount = NewEntryCount;
	Table->EntryStride = NewStride;
	Table->SubEntryStride = (NewOp == RDO_None) ? 0 : ValuesPerEntry;
	Table->TimeScale = (NewTimeScale > 0.0f) ? 1.0f / NewTimeScale : 0.0f;
	Table->TimeBias = MinIn;
	Table->Values.Empty( NewEntryCount * NewStride );
	Table->Values.AddZeroed( NewEntryCount * NewStride );
	for ( int32 SampleIndex = 0; SampleIndex < NewEntryCount; ++SampleIndex )
	{
		const float* InEntry1;
		const float* InEntry2;
		const float* OtherEntry1;
		const float* OtherEntry2;
		float InLerpAlpha;
		float OtherLerpAlpha;

		const float Time = MinIn + SampleIndex * NewTimeScale;
		InTable.GetEntry( Time, InEntry1, InEntry2, InLerpAlpha );
		OtherTable.GetEntry( Time, OtherEntry1, OtherEntry2, OtherLerpAlpha );

		// Scale sub-entry 1.
		for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
		{
			Table->Values[SampleIndex * NewStride + ValueIndex] =
				FMath::Lerp( InEntry1[ValueIndex], InEntry2[ValueIndex], InLerpAlpha ) +
				FMath::Lerp( OtherEntry1[ValueIndex % OtherValuesPerEntry], OtherEntry2[ValueIndex % OtherValuesPerEntry], OtherLerpAlpha );
		}

		// Scale sub-entry 2 if needed. 
		if ( NewOp != RDO_None )
		{
			InEntry1 += InTable.SubEntryStride;
			InEntry2 += InTable.SubEntryStride;
			OtherEntry1 += OtherTable.SubEntryStride;
			OtherEntry2 += OtherTable.SubEntryStride;

			for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
			{
				Table->Values[SampleIndex * NewStride + ValuesPerEntry + ValueIndex] =
					FMath::Lerp( InEntry1[ValueIndex], InEntry2[ValueIndex], InLerpAlpha ) +
					FMath::Lerp( OtherEntry1[ValueIndex % OtherValuesPerEntry], OtherEntry2[ValueIndex % OtherValuesPerEntry], OtherLerpAlpha );
			}
		}
	}
}

/**
 * Computes the L2 norm between the samples in ValueCount dimensional space.
 * @param Values1 - Array of ValueCount values.
 * @param Values2 - Array of ValueCount values.
 * @param ValueCount - The number of values in the two arrays.
 * @returns the L2 norm of the difference of the vectors represented by the two float arrays.
 */
static float ComputeSampleDistance( const float* Values1, const float* Values2, int32 ValueCount )
{
	float Dist = 0.0f;
	for ( int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex )
	{
		const float Diff = Values1[ValueIndex] - Values2[ValueIndex];
		Dist += (Diff * Diff);
	}
	return FMath::Sqrt( Dist );
}

/**
 * Computes the chordal distance between the curves represented by the two tables.
 * @param Table1 - The first table to compare.
 * @param Table2 - The second table to compare.
 * @param MinIn - The time at which to begin comparing.
 * @param MaxIn - The time at which to stop comparing.
 * @param SampleCount - The number of samples to use.
 * @returns the chordal distance representing the error introduced by substituting one table for the other.
 */
static float ComputeLookupTableError( const FDistributionLookupTable& InTable1, const FDistributionLookupTable& InTable2, float MinIn, float MaxIn, int32 SampleCount )
{
	check( InTable1.EntryStride == InTable2.EntryStride );
	check( InTable1.SubEntryStride == InTable2.SubEntryStride );
	check( SampleCount > 0 );

	const FDistributionLookupTable* Table1 = (InTable2.EntryCount > InTable1.EntryCount) ? &InTable2 : &InTable1;
	const FDistributionLookupTable* Table2 = (Table1 == &InTable1) ? &InTable2 : &InTable1;
	const int32 ValuesPerEntry = Table1->GetValuesPerEntry();
	const float TimeStep = (SampleCount > 1) ? ((MaxIn - MinIn) / (float)(SampleCount-1)) : 0.0f;

	float Values1[4] = {0};
	float Values2[4] = {0};
	float Error = 0.0f;
	float Time = MinIn;
	for ( int32 SampleIndex = 0; SampleIndex < SampleCount; ++SampleIndex, Time += TimeStep )
	{
		const float* Table1Entry1 = NULL;
		const float* Table1Entry2 = NULL;
		float Table1LerpAlpha = 0.0f;
		const float* Table2Entry1 = NULL;
		const float* Table2Entry2 = NULL;
		float Table2LerpAlpha = 0.0f;

		Table1->GetEntry( Time, Table1Entry1, Table1Entry2, Table1LerpAlpha );
		Table2->GetEntry( Time, Table2Entry1, Table2Entry2, Table2LerpAlpha );
		for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
		{
			Values1[ValueIndex] = FMath::Lerp( Table1Entry1[ValueIndex], Table1Entry2[ValueIndex], Table1LerpAlpha );
			Values2[ValueIndex] = FMath::Lerp( Table2Entry1[ValueIndex], Table2Entry2[ValueIndex], Table2LerpAlpha );
		}
		Error = FMath::Max<float>( Error, ComputeSampleDistance( Values1, Values2, ValuesPerEntry ) );

		if ( Table1->SubEntryStride > 0 )
		{
			Table1Entry1 += Table1->SubEntryStride;
			Table1Entry2 += Table1->SubEntryStride;
			Table2Entry1 += Table2->SubEntryStride;
			Table2Entry2 += Table2->SubEntryStride;
			for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
			{
				Values1[ValueIndex] = FMath::Lerp( Table1Entry1[ValueIndex], Table1Entry2[ValueIndex], Table1LerpAlpha );
				Values2[ValueIndex] = FMath::Lerp( Table2Entry1[ValueIndex], Table2Entry2[ValueIndex], Table2LerpAlpha );
			}
			Error = FMath::Max<float>( Error, ComputeSampleDistance( Values1, Values2, ValuesPerEntry ) );
		}
	}
	return Error;
}

/**
 * Resamples a lookup table.
 * @param OutTable - The resampled table.
 * @param InTable - The table to be resampled.
 * @param MinIn - The time at which to begin resampling.
 * @param MaxIn - The time at which to stop resampling.
 * @param SampleCount - The number of samples to use.
 */
static void ResampleLookupTable( FDistributionLookupTable* OutTable, const FDistributionLookupTable& InTable, float MinIn, float MaxIn, int32 SampleCount )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	const int32 Stride = InTable.EntryStride;
	const float OneOverTimeScale = (InTable.TimeScale == 0.0f) ? 0.0f : 1.0f / InTable.TimeScale;
	const float TimeScale = (SampleCount > 1) ? ((MaxIn - MinIn) / (float)(SampleCount-1)) : 0.0f;

	// Build the resampled table.
	OutTable->Op = InTable.Op;
	OutTable->EntryCount = SampleCount;
	OutTable->EntryStride = InTable.EntryStride;
	OutTable->SubEntryStride = InTable.SubEntryStride;
	OutTable->TimeBias = MinIn;
	OutTable->TimeScale = (TimeScale > 0.0f) ? (1.0f / TimeScale) : 0.0f;
	OutTable->Values.Empty( SampleCount * InTable.EntryStride );
	OutTable->Values.AddZeroed( SampleCount * InTable.EntryStride );

	// Resample entries in the table.
	for ( int32 SampleIndex = 0; SampleIndex < SampleCount; ++SampleIndex )
	{
		const float* Entry1 = NULL;
		const float* Entry2 = NULL;
		float LerpAlpha = 0.0f;
		const float Time = MinIn + TimeScale * SampleIndex;
		InTable.GetEntry( Time, Entry1, Entry2, LerpAlpha );
		for ( int32 ValueIndex = 0; ValueIndex < Stride; ++ValueIndex )
		{
			OutTable->Values[SampleIndex * Stride + ValueIndex] =
				FMath::Lerp( Entry1[ValueIndex], Entry2[ValueIndex], LerpAlpha );
		}
	}
}

/**
 * Optimizes a lookup table using the minimum number of samples required to represent the distribution.
 * @param Table - The lookup table to optimize.
 * @param ErrorThreshold - Threshold at which the lookup table is considered good enough.
 */
static void OptimizeLookupTable( FDistributionLookupTable* Table, float ErrorThreshold )
{
	check(IsInGameThread() || IsInAsyncLoadingThread());
	check( Table != NULL );
	check( (Table->EntryCount & (Table->EntryCount-1)) == 0 );

	// Domain for the table.
	const float OneOverTimeScale = (Table->TimeScale == 0.0f) ? 0.0f : 1.0f / Table->TimeScale;
	const float MinIn = Table->TimeBias;
	const float MaxIn = Table->TimeBias + (Table->EntryCount-1) * OneOverTimeScale;

	// Duplicate the table.
	FDistributionLookupTable OriginalTable = *Table;

	// Resample the lookup table until error is reduced to an acceptable level.
	const int32 MinSampleCount = 1;
	const int32 MaxSampleCount = LOOKUP_TABLE_MAX_SAMPLES;
	for ( int32 SampleCount = MinSampleCount; SampleCount < MaxSampleCount; SampleCount <<= 1 )
	{
		ResampleLookupTable( Table, OriginalTable, MinIn, MaxIn, SampleCount );
		if ( ComputeLookupTableError( *Table, OriginalTable, MinIn, MaxIn, LOOKUP_TABLE_MAX_SAMPLES ) < ErrorThreshold )
		{
			return;
		}
	}

	// The original table is optimal.
	*Table = OriginalTable;
}

void FRawDistribution::GetValue1(float Time, float* Value, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	switch (LookupTable.Op)
	{
	case RDO_None:
		GetValue1None(Time,Value);
		break;
	case RDO_Extreme:
		GetValue1Extreme(Time,Value,Extreme,InRandomStream);
		break;
	case RDO_Random:
		GetValue1Random(Time,Value,InRandomStream);
		break;
	default: // compiler complains
		checkSlow(0);
		*Value = 0.0f;
		break;
	}
}

void FRawDistribution::GetValue3(float Time, float* Value, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	switch (LookupTable.Op)
	{
	case RDO_None:
		GetValue3None(Time,Value);
		break;
	case RDO_Extreme:
		GetValue3Extreme(Time,Value,Extreme,InRandomStream);
		break;
	case RDO_Random:
		GetValue3Random(Time,Value,InRandomStream);
		break;
	}
}

void FRawDistribution::GetValue1Extreme(float Time, float* InValue, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	float* RESTRICT Value = InValue;
	const float* Entry1;
	const float* Entry2;
	float LerpAlpha = 0.0f;
	float RandValue = DIST_GET_RANDOM_VALUE(InRandomStream);
	LookupTable.GetEntry( Time, Entry1, Entry2, LerpAlpha );
	const float* RESTRICT NewEntry1 = Entry1;
	const float* RESTRICT NewEntry2 = Entry2;
	int32 InitialElement = ((Extreme > 0) || ((Extreme == 0) && (RandValue > 0.5f)));
	Value[0] = FMath::Lerp(NewEntry1[InitialElement + 0], NewEntry2[InitialElement + 0], LerpAlpha);
}

void FRawDistribution::GetValue3Extreme(float Time, float* InValue, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	float* RESTRICT Value = InValue;
	const float* Entry1;
	const float* Entry2;
	float LerpAlpha = 0.0f;
	float RandValue = DIST_GET_RANDOM_VALUE(InRandomStream);
	LookupTable.GetEntry( Time, Entry1, Entry2, LerpAlpha );
	const float* RESTRICT NewEntry1 = Entry1;
	const float* RESTRICT NewEntry2 = Entry2;
	int32 InitialElement = ((Extreme > 0) || ((Extreme == 0) && (RandValue > 0.5f)));
	InitialElement *= 3;
	float T0 = FMath::Lerp(NewEntry1[InitialElement + 0], NewEntry2[InitialElement + 0], LerpAlpha);
	float T1 = FMath::Lerp(NewEntry1[InitialElement + 1], NewEntry2[InitialElement + 1], LerpAlpha);
	float T2 = FMath::Lerp(NewEntry1[InitialElement + 2], NewEntry2[InitialElement + 2], LerpAlpha);
	Value[0] = T0;
	Value[1] = T1;
	Value[2] = T2;
}

void FRawDistribution::GetValue1Random(float Time, float* InValue, struct FRandomStream* InRandomStream) const
{
	float* RESTRICT Value = InValue;
	const float* Entry1;
	const float* Entry2;
	float LerpAlpha = 0.0f;
	float RandValue = DIST_GET_RANDOM_VALUE(InRandomStream);
	LookupTable.GetEntry( Time, Entry1, Entry2, LerpAlpha );
	const float* RESTRICT NewEntry1 = Entry1;
	const float* RESTRICT NewEntry2 = Entry2;
	float Value1,Value2;
	Value1 = FMath::Lerp(NewEntry1[0], NewEntry2[0], LerpAlpha);
	Value2 = FMath::Lerp(NewEntry1[1 + 0], NewEntry2[1 + 0], LerpAlpha);
	Value[0] = Value1 + (Value2 - Value1) * RandValue;
}

void FRawDistribution::GetValue3Random(float Time, float* InValue, struct FRandomStream* InRandomStream) const
{
	float* RESTRICT Value = InValue;
	const float* Entry1;
	const float* Entry2;
	float LerpAlpha = 0.0f;
	FVector RandValues;

	RandValues[0] = DIST_GET_RANDOM_VALUE(InRandomStream);
	RandValues[1] = DIST_GET_RANDOM_VALUE(InRandomStream);
	RandValues[2] = DIST_GET_RANDOM_VALUE(InRandomStream);
	switch(LookupTable.LockFlag)
	{
	case EDVLF_XY:
		RandValues.Y = RandValues.X;
		break;
	case EDVLF_XZ:
		RandValues.Z = RandValues.X;
		break;
	case EDVLF_YZ:
		RandValues.Z = RandValues.Y;
		break;
	case EDVLF_XYZ:
		RandValues.Y = RandValues.X;
		RandValues.Z = RandValues.X;
		break;
	}

	LookupTable.GetEntry( Time, Entry1, Entry2, LerpAlpha );
	const float* RESTRICT NewEntry1 = Entry1;
	const float* RESTRICT NewEntry2 = Entry2;
	float X0,X1;
	float Y0,Y1;
	float Z0,Z1;
	X0 = FMath::Lerp(NewEntry1[0], NewEntry2[0], LerpAlpha);
	Y0 = FMath::Lerp(NewEntry1[1], NewEntry2[1], LerpAlpha);
	Z0 = FMath::Lerp(NewEntry1[2], NewEntry2[2], LerpAlpha);
	X1 = FMath::Lerp(NewEntry1[3 + 0], NewEntry2[3 + 0], LerpAlpha);
	Y1 = FMath::Lerp(NewEntry1[3 + 1], NewEntry2[3 + 1], LerpAlpha);
	Z1 = FMath::Lerp(NewEntry1[3 + 2], NewEntry2[3 + 2], LerpAlpha);
	Value[0] = X0 + (X1 - X0) * RandValues[0];
	Value[1] = Y0 + (Y1 - Y0) * RandValues[1];
	Value[2] = Z0 + (Z1 - Z0) * RandValues[2];
}

void FRawDistribution::GetValue(float Time, float* Value, int32 NumCoords, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	checkSlow(NumCoords == 3 || NumCoords == 1);
	switch (LookupTable.Op)
	{
	case RDO_None:
		if (NumCoords == 1)
		{
			GetValue1None(Time,Value);
		}
		else
		{
			GetValue3None(Time,Value);
		}
		break;
	case RDO_Extreme:
		if (NumCoords == 1)
		{
			GetValue1Extreme(Time,Value,Extreme,InRandomStream);
		}
		else
		{
			GetValue3Extreme(Time,Value,Extreme,InRandomStream);
		}
		break;
	case RDO_Random:
		if (NumCoords == 1)
		{
			GetValue1Random(Time,Value,InRandomStream);
		}
		else
		{
			GetValue3Random(Time,Value,InRandomStream);
		}
		break;
	}
}

UObject* FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty(FStructProperty* Property, uint8* Data)
{
	// if the struct in this property is of type FRawDistributionFloat
	if (Property->Struct->GetFName() == NAME_RawDistributionFloat)
	{
		// then return the UDistribution pointed to by the FRawDistributionFloat
		return Property->ContainerPtrToValuePtr<FRawDistributionFloat>(Data)->Distribution;
	}
	// if the struct in this property is of type FRawDistributionVector
	else if (Property->Struct->GetFName() == NAME_RawDistributionVector)
	{
		// then return the UDistribution pointed to by the FRawDistributionVector
		return Property->ContainerPtrToValuePtr<FRawDistributionVector>(Data)->Distribution;
	}

	// if this wasn't a FRawDistribution*, return NULL
	return NULL;
}


#if WITH_EDITOR

void FRawDistributionFloat::Initialize()
{
	// Nothing to do if we don't have a distribution.
	if( Distribution == NULL )
	{
		return;
	}

	// does this FRawDist need updating? (if UDist is dirty or somehow the distribution wasn't dirty, but we have no data)
	bool bNeedsUpdating = false;
	if (Distribution->bIsDirty || (LookupTable.IsEmpty() && Distribution->CanBeBaked()))
	{
		if (!Distribution->bIsDirty)
		{
			UE_LOGF(LogDistributions,Log,"Somehow Distribution %ls wasn't dirty, but its FRawDistribution wasn't ever initialized!", *Distribution->GetFullName());
		}
		bNeedsUpdating = true;
	}
	// only initialize if we need to
	if (!bNeedsUpdating)
	{
		return;
	}
	if (!GIsEditor && !IsInGameThread() && !IsInAsyncLoadingThread())
	{
		return;
	}
	check(IsInGameThread() || IsInAsyncLoadingThread());

	// always empty out the lookup table
	LookupTable.Empty();

	// distribution is no longer dirty (if it was)
	// template objects aren't marked as dirty, because any UDists that uses this as an archetype, 
	// aren't the default values, and has already been saved, needs to know to build the FDist
	if (!Distribution->IsTemplate())
	{
		Distribution->bIsDirty = false;
	}

	// if the distribution can't be baked out, then we do nothing here
	if (!Distribution->CanBeBaked())
	{
		return;
	}

	// Build and optimize the lookup table.
	BuildLookupTable( &LookupTable, Distribution );
	OptimizeLookupTable( &LookupTable, LOOKUP_TABLE_ERROR_THRESHOLD );

	// fill out our min/max
	Distribution->GetOutRange(MinValue, MaxValue);
}
#endif // WITH_EDITOR

bool FRawDistributionFloat::IsCreated()
{
	return HasLookupTable(/*bInitializeIfNeeded=*/ false) || (Distribution != nullptr);
}

bool FRawDistributionVector::IsCreated()
{
	return HasLookupTable(/*bInitializeIfNeeded=*/ false) || (Distribution != nullptr);
}

float FRawDistributionFloat::GetValue(float F, UObject* Data, struct FRandomStream* InRandomStream)
{
	if (!HasLookupTable())
	{
		if (!Distribution)
		{
			return 0.0f;
		}
		return Distribution->GetValue(F, Data, InRandomStream);
	}

	// if we get here, we better have been initialized!
	check(!LookupTable.IsEmpty());

	float Value;
	FRawDistribution::GetValue1(F, &Value, 0, InRandomStream);
	return Value;
}

const FRawDistribution *FRawDistributionFloat::GetFastRawDistribution()
{
	if (!IsSimple() || !HasLookupTable())
	{
		return 0;
	}

	// if we get here, we better have been initialized!
	check(!LookupTable.IsEmpty());

	return this;
}

void FRawDistributionFloat::GetOutRange(float& MinOut, float& MaxOut)
{
	if (!HasLookupTable() && Distribution)
	{
		check(Distribution);
		Distribution->GetOutRange(MinOut, MaxOut);
	}
	else
	{
		MinOut = MinValue;
		MaxOut = MaxValue;
	}
}

void FRawDistributionFloat::InitLookupTable()
{
#if WITH_EDITOR
	// make sure it's up to date
	if(Distribution)
	{
		if( GIsEditor || Distribution->bIsDirty)
		{
			Distribution->ConditionalPostLoad();
			Initialize();
		}
	}
#endif
}

#if WITH_EDITOR
template <typename RawDistributionType>
bool HasBakedDistributionDataHelper(const UDistribution* GivenDistribution)
{
	if (UObject* Outer = GivenDistribution->GetOuter())
	{
		for (TFieldIterator<FProperty> It(Outer->GetClass()); It; ++It)
		{
			FProperty* Property = *It;

			if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
			{
				UObject* Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty(StructProp, reinterpret_cast<uint8*>(Outer));
				if (Distribution == GivenDistribution)
				{
					if (RawDistributionType* RawDistributionVector = StructProp->ContainerPtrToValuePtr<RawDistributionType>(reinterpret_cast<uint8*>(Outer)))
					{
						if (RawDistributionVector->HasLookupTable(false))
						{
							return true;	//We have baked data
						}
					}

					return false;
				}
			}
			else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
			{
				if (FStructProperty* InnerStructProp = CastField<FStructProperty>(ArrayProp->Inner))
				{
					FScriptArrayHelper ArrayHelper(ArrayProp, Property->ContainerPtrToValuePtr<void>(Outer));
					for (int32 Idx = 0; Idx < ArrayHelper.Num(); ++Idx)
					{
						for (FProperty* ArrayProperty = InnerStructProp->Struct->PropertyLink; ArrayProperty; ArrayProperty = ArrayProperty->PropertyLinkNext)
						{
							if (FStructProperty* ArrayStructProp = CastField<FStructProperty>(ArrayProperty))
							{
								UObject* Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty(ArrayStructProp, ArrayHelper.GetRawPtr(Idx));
								if (Distribution == GivenDistribution)
								{
									if (RawDistributionType* RawDistributionVector = ArrayStructProp->ContainerPtrToValuePtr<RawDistributionType>(ArrayHelper.GetRawPtr(Idx)))
									{
										if (RawDistributionVector->HasLookupTable(false))
										{
											return true;	//We have baked data
										}
									}

									return false;
								}
							}
						}
					}
				}
			}
		}
	}

	return false;
}
#endif

void UDistributionFloat::Serialize(FStructuredArchive::FRecord Record)
{
#if WITH_EDITOR
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	if (UnderlyingArchive.IsCooking() || UnderlyingArchive.IsSaving())
	{
		bBakedDataSuccesfully = HasBakedDistributionDataHelper<FRawDistributionFloat>(this);
	}
#endif

	Super::Serialize(Record);
}


bool UDistributionFloat::NeedsLoadForClient() const
{
#if DISTRIBUTIONS_BAKES_OUT
	if (CanBeBaked() && HasBakedSuccesfully())
	{
		return false;
	}
#endif

	return true;
}


bool UDistributionFloat::NeedsLoadForServer() const 
{
#if DISTRIBUTIONS_BAKES_OUT
	if (CanBeBaked() && HasBakedSuccesfully())
	{
		return false;
	}
#endif

	return true;
}

bool UDistributionFloat::NeedsLoadForEditorGame() const
{
	return true;
}

#if	WITH_EDITOR
void UDistributionFloat::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	bIsDirty = true;
}
#endif	// WITH_EDITOR

float UDistributionFloat::GetValue( float F, UObject* Data, struct FRandomStream* InRandomStream ) const
{
	return 0.0;
}

float UDistributionFloat::GetFloatValue(float F)
{
	return GetValue(F);
}

void UDistributionFloat::GetInRange(float& MinIn, float& MaxIn) const
{
	MinIn	= 0.0f;
	MaxIn	= 0.0f;
}

void UDistributionFloat::GetOutRange(float& MinOut, float& MaxOut) const
{
	MinOut	= 0.0f;
	MaxOut	= 0.0f;
}

//@todo.CONSOLE
uint32 UDistributionFloat::InitializeRawEntry(float Time, float* Values) const
{
	Values[0] = GetValue(Time);
	return 1;
}

#if WITH_EDITOR
void FRawDistributionVector::Initialize()
{
	// Nothing to do if we don't have a distribution.
	if( Distribution == NULL )
	{
		return;
	}

	// fill out our min/max
	Distribution->GetOutRange(MinValue, MaxValue);
	Distribution->GetRange(MinValueVec, MaxValueVec);

	// does this FRawDist need updating? (if UDist is dirty or somehow the distribution wasn't dirty, but we have no data)
	bool bNeedsUpdating = false;
	if (Distribution->bIsDirty || (LookupTable.IsEmpty() && Distribution->CanBeBaked()))
	{
		if (!Distribution->bIsDirty)
		{
			UE_LOGF(LogDistributions,Log,"Somehow Distribution %ls wasn't dirty, but its FRawDistribution wasn't ever initialized!", *Distribution->GetFullName());
		}
		bNeedsUpdating = true;
	}

	// only initialize if we need to
	if (!bNeedsUpdating)
	{
		return;
	}
	check(IsInGameThread() || IsInAsyncLoadingThread());

	// always empty out the lookup table
	LookupTable.Empty();

	// distribution is no longer dirty (if it was)
	// template objects aren't marked as dirty, because any UDists that uses this as an archetype, 
	// aren't the default values, and has already been saved, needs to know to build the FDist
	if (!Distribution->IsTemplate())
	{
		Distribution->bIsDirty = false;	
	}

	// if the distribution can't be baked out, then we do nothing here
	if (!Distribution->CanBeBaked())
	{
		return;
	}

	// Build and optimize the lookup table.
	BuildLookupTable( &LookupTable, Distribution );
	const float MinIn = LookupTable.TimeBias;
	const float MaxIn = MinIn + (LookupTable.EntryCount-1) * (LookupTable.TimeScale == 0.0f ? 0.0f : (1.0f / LookupTable.TimeScale));
	OptimizeLookupTable( &LookupTable, LOOKUP_TABLE_ERROR_THRESHOLD );

}
#endif

FVector FRawDistributionVector::GetValue(float F, UObject* Data, int32 Extreme, struct FRandomStream* InRandomStream)
{
	if (!HasLookupTable())
	{
		if (!Distribution)
		{
			return FVector::ZeroVector;
		}
		return Distribution->GetValue(F, Data, Extreme, InRandomStream);
	}

	// if we get here, we better have been initialized!
	check(!LookupTable.IsEmpty());

	FVector3f Value;
	FRawDistribution::GetValue3(F, &Value.X, Extreme, InRandomStream);
	return (FVector)Value;
}

const FRawDistribution *FRawDistributionVector::GetFastRawDistribution()
{
	if (!IsSimple() || !HasLookupTable()) 
	{
		return 0;
	}

	// if we get here, we better have been initialized!
	check(!LookupTable.IsEmpty());

	return this;
}

void FRawDistributionVector::GetOutRange(float& MinOut, float& MaxOut)
{
	if (!HasLookupTable() && Distribution)
	{
		check(Distribution);
		Distribution->GetOutRange(MinOut, MaxOut);
	}
	else
	{
		MinOut = MinValue;
		MaxOut = MaxValue;
	}
}

void FRawDistributionVector::GetRange(FVector& MinOut, FVector& MaxOut)
{
	if (Distribution)
	{
		check(Distribution);
		Distribution->GetRange(MinOut, MaxOut);
	}
	else
	{
		MinOut = MinValueVec;
		MaxOut = MaxValueVec;
	}
}

void FRawDistributionVector::InitLookupTable()
{
#if WITH_EDITOR
	// make sure it's up to date
	if( GIsEditor || (Distribution && Distribution->bIsDirty) )
	{
		Initialize();
	}
#endif
}

void UDistributionVector::Serialize(FStructuredArchive::FRecord Record)
{
#if WITH_EDITOR
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	if (UnderlyingArchive.IsCooking() || UnderlyingArchive.IsSaving())
	{
		bBakedDataSuccesfully = HasBakedDistributionDataHelper<FRawDistributionVector>(this);
	}
#endif

	Super::Serialize(Record);
}

bool UDistributionVector::NeedsLoadForClient() const
{
#if DISTRIBUTIONS_BAKES_OUT
	if (CanBeBaked() && HasBakedSuccesfully())
	{
		return false;
	}
#endif

	return true;
}

bool UDistributionVector::NeedsLoadForServer() const 
{
#if DISTRIBUTIONS_BAKES_OUT
	if (CanBeBaked() && HasBakedSuccesfully())
	{
		return false;
	}
#endif

	return true;
}

bool UDistributionVector::NeedsLoadForEditorGame() const
{
	return true;
}

#if	WITH_EDITOR
void UDistributionVector::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	bIsDirty = true;
}
#endif // WITH_EDITOR

FVector UDistributionVector::GetValue(float F, UObject* Data, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	return FVector::ZeroVector;
}

FVector UDistributionVector::GetVectorValue(float F)
{
	return GetValue(F);
}

void UDistributionVector::GetInRange(float& MinIn, float& MaxIn) const
{
	MinIn	= 0.0f;
	MaxIn	= 0.0f;
}

void UDistributionVector::GetOutRange(float& MinOut, float& MaxOut) const
{
	MinOut	= 0.0f;
	MaxOut	= 0.0f;
}

void UDistributionVector::GetRange(FVector& OutMin, FVector& OutMax) const
{
	OutMin	= FVector::ZeroVector;
	OutMax	= FVector::ZeroVector;
}

//@todo.CONSOLE
uint32 UDistributionVector::InitializeRawEntry(float Time, float* Values) const
{
	FVector Value = GetValue(Time);
	Values[0] = Value.X;
	Values[1] = Value.Y;
	Values[2] = Value.Z;
	return 3;
}

FFloatDistribution::FFloatDistribution()
{
	BuildZeroLookupTable( &LookupTable, 1 );
}


FVectorDistribution::FVectorDistribution()
{
	BuildZeroLookupTable( &LookupTable, 3 );
}


FVector4Distribution::FVector4Distribution()
{
	BuildZeroLookupTable( &LookupTable, 4 );
}

//@todo.CONSOLE
void FComposableDistribution::BuildFloat( FFloatDistribution& OutDistribution, const FComposableFloatDistribution& X )
{
	checkDistribution( X.LookupTable.GetValuesPerEntry() == 1 );

	OutDistribution.LookupTable = X.LookupTable;
	OptimizeLookupTable( &OutDistribution.LookupTable, LOOKUP_TABLE_ERROR_THRESHOLD );
}

void FComposableDistribution::BuildVector( FVectorDistribution& OutDistribution, const FComposableVectorDistribution& XYZ )
{
	checkDistribution( XYZ.LookupTable.GetValuesPerEntry() == 3 );

	OutDistribution.LookupTable = XYZ.LookupTable;
	OptimizeLookupTable( &OutDistribution.LookupTable, LOOKUP_TABLE_ERROR_THRESHOLD );
}

void FComposableDistribution::BuildVector4( FVector4Distribution& OutDistribution, const class FComposableVectorDistribution& XYZ, const class FComposableFloatDistribution& W )
{
	checkDistribution( XYZ.LookupTable.GetValuesPerEntry() == 3 );
	checkDistribution( W.LookupTable.GetValuesPerEntry() == 1 );

	OutDistribution.LookupTable = XYZ.LookupTable;
	AppendLookupTable( &OutDistribution.LookupTable, W.LookupTable );
	OptimizeLookupTable( &OutDistribution.LookupTable, LOOKUP_TABLE_ERROR_THRESHOLD );
}


void FComposableDistribution::BuildVector4( 
	class FVector4Distribution& OutDistribution,
	const class FComposableVectorDistribution& XY0,
	const class FComposableFloatDistribution& Z,
	const class FComposableFloatDistribution& W )
{
	checkDistribution( XY0.LookupTable.GetValuesPerEntry() == 3 );
	checkDistribution( Z.LookupTable.GetValuesPerEntry() == 1 );
	checkDistribution( W.LookupTable.GetValuesPerEntry() == 1 );

	OutDistribution.LookupTable = XY0.LookupTable;
	SliceLookupTable( &OutDistribution.LookupTable, 2 );
	AppendLookupTable( &OutDistribution.LookupTable, Z.LookupTable );
	AppendLookupTable( &OutDistribution.LookupTable, W.LookupTable );
	OptimizeLookupTable( &OutDistribution.LookupTable, LOOKUP_TABLE_ERROR_THRESHOLD );
}

void FComposableDistribution::BuildVector4( 
	FVector4Distribution& OutDistribution,
	const class FComposableFloatDistribution& X,
	const class FComposableFloatDistribution& Y,
	const class FComposableFloatDistribution& Z,
	const class FComposableFloatDistribution& W )
{
	checkDistribution( X.LookupTable.GetValuesPerEntry() == 1 );
	checkDistribution( Y.LookupTable.GetValuesPerEntry() == 1 );
	checkDistribution( Z.LookupTable.GetValuesPerEntry() == 1 );
	checkDistribution( W.LookupTable.GetValuesPerEntry() == 1 );

	OutDistribution.LookupTable = X.LookupTable;
	AppendLookupTable( &OutDistribution.LookupTable, Y.LookupTable );
	AppendLookupTable( &OutDistribution.LookupTable, Z.LookupTable );
	AppendLookupTable( &OutDistribution.LookupTable, W.LookupTable );
	OptimizeLookupTable( &OutDistribution.LookupTable, LOOKUP_TABLE_ERROR_THRESHOLD );
}


void FComposableDistribution::QuantizeVector4(
	TArray<FColor>& OutQuantizedSamples,
	FVector4& OutScale,
	FVector4& OutBias,
	const FVector4Distribution& Distribution )
{
	FVector4 Mins( +FLT_MAX, +FLT_MAX, +FLT_MAX, +FLT_MAX );
	FVector4 Maxs( -FLT_MAX, -FLT_MAX, -FLT_MAX, -FLT_MAX );
	const FDistributionLookupTable& Table = Distribution.LookupTable;
	const int32 EntryCount = Table.EntryCount;
	const int32 EntryStride = Table.EntryStride;
	const float* RESTRICT Values = Table.Values.GetData();
	
	// First find the minimum and maximum values for each channel at each sample.
	for ( int32 EntryIndex = 0; EntryIndex < EntryCount; ++EntryIndex )
	{
		Mins.X = FMath::Min<float>( Mins.X, Values[0] );
		Mins.Y = FMath::Min<float>( Mins.Y, Values[1] );
		Mins.Z = FMath::Min<float>( Mins.Z, Values[2] );
		Mins.W = FMath::Min<float>( Mins.W, Values[3] );

		Maxs.X = FMath::Max<float>( Maxs.X, Values[0] );
		Maxs.Y = FMath::Max<float>( Maxs.Y, Values[1] );
		Maxs.Z = FMath::Max<float>( Maxs.Z, Values[2] );
		Maxs.W = FMath::Max<float>( Maxs.W, Values[3] );

		Values += EntryStride;
	}

	// Compute scale and bias.
	const FVector4 Scale( Maxs - Mins );
	const FVector4 InvScale(
		(Scale.X > UE_KINDA_SMALL_NUMBER ? (1.0f / Scale.X) : 0.0f) * 255.0f,
		(Scale.Y > UE_KINDA_SMALL_NUMBER ? (1.0f / Scale.Y) : 0.0f) * 255.0f,
		(Scale.Z > UE_KINDA_SMALL_NUMBER ? (1.0f / Scale.Z) : 0.0f) * 255.0f,
		(Scale.W > UE_KINDA_SMALL_NUMBER ? (1.0f / Scale.W) : 0.0f) * 255.0f
		);
	const FVector4 Bias( Mins );

	// If there is only one entry in the table, we don't need any samples at all.
	if ( EntryCount == 1 )
	{
		OutScale = Scale;
		OutBias = Bias;
		return;
	}

	// Now construct the quantized samples.
	OutQuantizedSamples.Empty( EntryCount );
	OutQuantizedSamples.AddUninitialized( EntryCount );
	FColor* RESTRICT QuantizedValues = OutQuantizedSamples.GetData();
	Values = Table.Values.GetData();
	for ( int32 EntryIndex = 0; EntryIndex < EntryCount; ++EntryIndex )
	{
		QuantizedValues->R = FMath::Clamp<int32>( FMath::TruncToInt( (Values[0] - Bias.X) * InvScale.X ), 0, 255 );
		QuantizedValues->G = FMath::Clamp<int32>( FMath::TruncToInt( (Values[1] - Bias.Y) * InvScale.Y ), 0, 255 );
		QuantizedValues->B = FMath::Clamp<int32>( FMath::TruncToInt( (Values[2] - Bias.Z) * InvScale.Z ), 0, 255 );
		QuantizedValues->A = FMath::Clamp<int32>( FMath::TruncToInt( (Values[3] - Bias.W) * InvScale.W ), 0, 255 );
		Values += EntryStride;
		QuantizedValues++;
	}
	OutScale = Scale;
	OutBias = Bias;
}

FComposableFloatDistribution::FComposableFloatDistribution()
{
	BuildZeroLookupTable( &LookupTable, 1 );
}

void FComposableFloatDistribution::Initialize( const UDistributionFloat* FloatDistribution )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
	if ( FloatDistribution != NULL && FloatDistribution->CanBeBaked() )
	{
		BuildLookupTable( &LookupTable, FloatDistribution );
	}
	else
	{
		BuildZeroLookupTable( &LookupTable, 1 );
	}
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
}

void FComposableFloatDistribution::InitializeWithConstant( float Value )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
	BuildConstantLookupTable( &LookupTable, 1, &Value );
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
}

void FComposableFloatDistribution::ScaleByConstant( float Scale )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
	ScaleLookupTableByConstant( &LookupTable, Scale );
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
}

void FComposableFloatDistribution::ScaleByDistribution( const UDistributionFloat* FloatDistribution )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
	if ( FloatDistribution && FloatDistribution->CanBeBaked() )
	{
		FDistributionLookupTable TableToScaleBy;
		BuildLookupTable( &TableToScaleBy, FloatDistribution );
		ScaleLookupTableByLookupTable( &LookupTable, TableToScaleBy );
	}
	else
	{
		BuildZeroLookupTable( &LookupTable, 1 );
	}
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
}

void FComposableFloatDistribution::AddDistribution( const UDistributionFloat* FloatDistribution )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
	if ( FloatDistribution && FloatDistribution->CanBeBaked() )
	{
		FDistributionLookupTable TableToAdd;
		BuildLookupTable( &TableToAdd, FloatDistribution );
		AddLookupTableToLookupTable( &LookupTable, TableToAdd );
	}
	checkDistribution( LookupTable.GetValuesPerEntry() == 1 );
}

void FComposableFloatDistribution::Normalize( float* OutScale, float* OutBias )
{
	float MinValue, MaxValue;
	float InvScale, InvBias;

	LookupTable.GetRange( &MinValue, &MaxValue );
	*OutBias = MinValue;
	InvBias = -MinValue;
	*OutScale = MaxValue - MinValue;
	InvScale = (FMath::Abs(MaxValue - MinValue) > UE_SMALL_NUMBER) ? (1.0f / (MaxValue - MinValue)) : 1.0f;

	AddConstantToLookupTable( &LookupTable, &InvBias, 1 );
	ScaleLookupTableByConstant( &LookupTable, InvScale );
}

void FComposableFloatDistribution::Resample( float MinIn, float MaxIn )
{
	FDistributionLookupTable OldTable = LookupTable;
	ResampleLookupTable( &LookupTable, OldTable, MinIn, MaxIn, LOOKUP_TABLE_MAX_SAMPLES );
}


FComposableVectorDistribution::FComposableVectorDistribution()
{
	const float Zero[3] = {0};
	BuildZeroLookupTable( &LookupTable, 3 );
}

void FComposableVectorDistribution::Initialize( const UDistributionVector* VectorDistribution )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
	if ( VectorDistribution != NULL && VectorDistribution->CanBeBaked() )
	{
		BuildLookupTable( &LookupTable, VectorDistribution );
	}
	else
	{
		BuildZeroLookupTable( &LookupTable, 3 );
	}
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::InitializeWithConstant( const FVector& Value )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
	FVector3f AsFloat = (FVector3f)Value;
	BuildConstantLookupTable(&LookupTable, 3, (float*)&AsFloat);
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::ScaleByConstant( float Scale )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
	ScaleLookupTableByConstant(&LookupTable, Scale);
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::ScaleByConstantVector( const FVector& Scale )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
	FVector3f AsFloat = (FVector3f)Scale;
	ScaleLookupTableByConstants(&LookupTable, (const float*)&AsFloat, 3);
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::AddConstantVector( const FVector& Value )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
	FVector3f AsFloat = (FVector3f)Value;
	AddConstantToLookupTable( &LookupTable, (const float*)&AsFloat, 3 );
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::ScaleByDistribution( const UDistributionFloat* FloatDistribution )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
	if ( FloatDistribution && FloatDistribution->CanBeBaked() )
	{
		FDistributionLookupTable TableToScaleBy;
		BuildLookupTable( &TableToScaleBy, FloatDistribution );
		ScaleLookupTableByLookupTable( &LookupTable, TableToScaleBy );
	}
	else
	{
		BuildZeroLookupTable( &LookupTable, 3 );
	}
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::ScaleByVectorDistribution( const UDistributionVector* VectorDistribution )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
	if ( VectorDistribution && VectorDistribution->CanBeBaked() )
	{
		FDistributionLookupTable TableToScaleBy;
		BuildLookupTable( &TableToScaleBy, VectorDistribution );
		ScaleLookupTableByLookupTable( &LookupTable, TableToScaleBy );
	}
	else
	{
		BuildZeroLookupTable( &LookupTable, 3 );
	}
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::AddDistribution( const UDistributionVector* VectorDistribution )
{
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
	if ( VectorDistribution && VectorDistribution->CanBeBaked() )
	{
		FDistributionLookupTable TableToAdd;
		BuildLookupTable( &TableToAdd, VectorDistribution );
		AddLookupTableToLookupTable( &LookupTable, TableToAdd );
	}
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::Splat( int32 ChannelIndex )
{
	check( ChannelIndex >= 0 && ChannelIndex <= 3 );
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );

	const int32 ValueCount = LookupTable.Values.Num();
	for ( int32 Index = 0; Index < ValueCount; Index += 3 )
	{
		float* Entry = &LookupTable.Values[Index];
		const float ValueToSplat = Entry[ChannelIndex];
		for ( int32 ValueIndex = 0; ValueIndex < 3; ++ValueIndex )
		{
			Entry[ValueIndex] = ValueToSplat;
		}
	}
	checkDistribution( LookupTable.GetValuesPerEntry() == 3 );
}

void FComposableVectorDistribution::Resample( float MinIn, float MaxIn )
{
	FDistributionLookupTable OldTable = LookupTable;
	ResampleLookupTable( &LookupTable, OldTable, MinIn, MaxIn, LOOKUP_TABLE_MAX_SAMPLES );
}

UDistributionFloatConstant::UDistributionFloatConstant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UDistributionFloatConstant::PostInitProperties()
{
	Super::PostInitProperties();	
	if (HasAnyFlags(RF_NeedLoad) && (GetOuter()->IsA(UParticleModule::StaticClass()) || GetOuter()->IsA(USoundNode::StaticClass())))
	{
		// Set to a bogus value for distributions created before VER_UE4_MOVE_DISTRIBUITONS_TO_POSTINITPROPS
		// to be able to restore to the previous default value.
		Constant = UDistribution::DefaultValue;
	}
}

void UDistributionFloatConstant::PostLoad()
{
	Super::PostLoad();
	if (Constant == UDistribution::DefaultValue)
	{
		// Reset to default for distributions created before VER_UE4_MOVE_DISTRIBUITONS_TO_POSTINITPROPS.
		Constant = 0.0f;
	}
}

float UDistributionFloatConstant::GetValue( float F, UObject* Data, struct FRandomStream* InRandomStream ) const
{
	return Constant;
}


int32 UDistributionFloatConstant::GetNumKeys() const
{
	return 1;
}

int32 UDistributionFloatConstant::GetNumSubCurves() const
{
	return 1;
}

float UDistributionFloatConstant::GetKeyIn(int32 KeyIndex)
{
	check( KeyIndex == 0 );
	return 0.f;
}

float UDistributionFloatConstant::GetKeyOut(int32 SubIndex, int32 KeyIndex)
{
	check( SubIndex == 0 );
	check( KeyIndex == 0 );
	return Constant;
}

FColor UDistributionFloatConstant::GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor)
{
	// There can be only be one sub-curve for this distribution.
	check( SubIndex == 0 );
	// There can be only be one key for this distribution.
	check( KeyIndex == 0 );

	// Always return RED since there is only one key
	return FColor::Red;
}

void UDistributionFloatConstant::GetInRange(float& MinIn, float& MaxIn) const
{
	MinIn = 0.f;
	MaxIn = 0.f;
}

void UDistributionFloatConstant::GetOutRange(float& MinOut, float& MaxOut) const
{
	MinOut = Constant;
	MaxOut = Constant;
}

EInterpCurveMode UDistributionFloatConstant::GetKeyInterpMode(int32 KeyIndex) const
{
	check( KeyIndex == 0 );
	return CIM_Constant;
}

void UDistributionFloatConstant::GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const
{
	check( SubIndex == 0 );
	check( KeyIndex == 0 );
	ArriveTangent = 0.f;
	LeaveTangent = 0.f;
}

float UDistributionFloatConstant::EvalSub(int32 SubIndex, float InVal)
{
	check(SubIndex == 0);
	return Constant;
}

int32 UDistributionFloatConstant::CreateNewKey(float KeyIn)
{	
	return 0;
}

void UDistributionFloatConstant::DeleteKey(int32 KeyIndex)
{
	check( KeyIndex == 0 );
}

int32 UDistributionFloatConstant::SetKeyIn(int32 KeyIndex, float NewInVal)
{
	check( KeyIndex == 0 );
	return 0;
}

void UDistributionFloatConstant::SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) 
{
	check( SubIndex == 0 );
	check( KeyIndex == 0 );
	Constant = NewOutVal;

	bIsDirty = true;
}

void UDistributionFloatConstant::SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) 
{
	check( KeyIndex == 0 );
}

void UDistributionFloatConstant::SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent)
{
	check( SubIndex == 0 );
	check( KeyIndex == 0 );
}

UDistributionFloatConstantCurve::UDistributionFloatConstantCurve(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

float UDistributionFloatConstantCurve::GetValue( float F, UObject* Data, struct FRandomStream* InRandomStream ) const
{
	return ConstantCurve.Eval(F, 0.f);
}


int32 UDistributionFloatConstantCurve::GetNumKeys() const
{
	return ConstantCurve.Points.Num();
}

int32 UDistributionFloatConstantCurve::GetNumSubCurves() const
{
	return 1;
}

float UDistributionFloatConstantCurve::GetKeyIn(int32 KeyIndex)
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	return ConstantCurve.Points[KeyIndex].InVal;
}

float UDistributionFloatConstantCurve::GetKeyOut(int32 SubIndex, int32 KeyIndex)
{
	check( SubIndex == 0 );
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	return ConstantCurve.Points[KeyIndex].OutVal;
}

FColor UDistributionFloatConstantCurve::GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor)
{
	// There can be only be one sub-curve for this distribution.
	check( SubIndex == 0 );
	// There can be only be one key for this distribution.
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );

	// Always return RED since there is only one sub-curve.
	return FColor::Red;
}

void UDistributionFloatConstantCurve::GetInRange(float& MinIn, float& MaxIn) const
{
	if(ConstantCurve.Points.Num() == 0)
	{
		MinIn = 0.f;
		MaxIn = 0.f;
	}
	else
	{
		float Min = UE_BIG_NUMBER;
		float Max = -UE_BIG_NUMBER;
		for (int32 Index = 0; Index < ConstantCurve.Points.Num(); Index++)
		{
			float Value = ConstantCurve.Points[Index].InVal;
			if (Value < Min)
			{
				Min = Value;
			}
			if (Value > Max)
			{
				Max = Value;
			}
		}
		MinIn = Min;
		MaxIn = Max;
	}
}

void UDistributionFloatConstantCurve::GetOutRange(float& MinOut, float& MaxOut) const
{
	ConstantCurve.CalcBounds(MinOut, MaxOut, 0.f);
}

EInterpCurveMode UDistributionFloatConstantCurve::GetKeyInterpMode(int32 KeyIndex) const
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	return ConstantCurve.Points[KeyIndex].InterpMode;
}

void UDistributionFloatConstantCurve::GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const
{
	check( SubIndex == 0 );
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent;
	LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent;
}

float UDistributionFloatConstantCurve::EvalSub(int32 SubIndex, float InVal)
{
	check(SubIndex == 0);
	return ConstantCurve.Eval(InVal, 0.f);
}

int32 UDistributionFloatConstantCurve::CreateNewKey(float KeyIn)
{
	float NewKeyOut = ConstantCurve.Eval(KeyIn, 0.f);
	int32 NewPointIndex = ConstantCurve.AddPoint(KeyIn, NewKeyOut);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;

	return NewPointIndex;
}

void UDistributionFloatConstantCurve::DeleteKey(int32 KeyIndex)
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	ConstantCurve.Points.RemoveAt(KeyIndex);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

int32 UDistributionFloatConstantCurve::SetKeyIn(int32 KeyIndex, float NewInVal)
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	int32 NewPointIndex = ConstantCurve.MovePoint(KeyIndex, NewInVal);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;

	return NewPointIndex;
}

void UDistributionFloatConstantCurve::SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) 
{
	check( SubIndex == 0 );
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	ConstantCurve.Points[KeyIndex].OutVal = NewOutVal;
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

void UDistributionFloatConstantCurve::SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) 
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	ConstantCurve.Points[KeyIndex].InterpMode = NewMode;
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

void UDistributionFloatConstantCurve::SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent)
{
	check( SubIndex == 0 );
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	ConstantCurve.Points[KeyIndex].ArriveTangent = ArriveTangent;
	ConstantCurve.Points[KeyIndex].LeaveTangent = LeaveTangent;

	bIsDirty = true;
}


UDistributionFloatUniform::UDistributionFloatUniform(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UDistributionFloatUniform::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_NeedLoad) && (GetOuter()->IsA(UParticleModule::StaticClass()) || GetOuter()->IsA(USoundNode::StaticClass())))
	{
		// Set to a bogus value for distributions created before VER_UE4_MOVE_DISTRIBUITONS_TO_POSTINITPROPS
		// to be able to restore to the previous default value.
		Min = UDistribution::DefaultValue;
		Max = UDistribution::DefaultValue;
	}
}

void UDistributionFloatUniform::PostLoad()
{
	Super::PostLoad();
	// Reset to default for distributions created before VER_UE4_MOVE_DISTRIBUITONS_TO_POSTINITPROPS.
	if (Min == UDistribution::DefaultValue)
	{
		Min = 0.0f;
	}
	if (Max == UDistribution::DefaultValue)
	{
		Max = 0.0f;
	}
}

float UDistributionFloatUniform::GetValue( float F, UObject* Data, struct FRandomStream* InRandomStream ) const
{
	return Max + (Min - Max) * DIST_GET_RANDOM_VALUE(InRandomStream);
}

//@todo.CONSOLE
ERawDistributionOperation UDistributionFloatUniform::GetOperation() const
{
	if (Min == Max)
	{
		// This may as well be a constant - don't bother doing the FMath::SRand scaling on it.
		return RDO_None;
	}
	return RDO_Random;
}
	
uint32 UDistributionFloatUniform::InitializeRawEntry(float Time, float* Values) const
{
	Values[0] = Min;
	Values[1] = Max;
	return 2;
}

int32 UDistributionFloatUniform::GetNumKeys() const
{
	return 1;
}

int32 UDistributionFloatUniform::GetNumSubCurves() const
{
	return 2;
}

FColor UDistributionFloatUniform::GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const
{
	// Check for array out of bounds because it will crash the program
	check(SubCurveIndex >= 0);
	check(SubCurveIndex < GetNumSubCurves());

	FColor ButtonColor;

	switch(SubCurveIndex)
	{
	case 0:
		// Red
		ButtonColor = bIsSubCurveHidden ? FColor(32, 0, 0) : FColor::Red;
		break;
	case 1:
		// Green
		ButtonColor = bIsSubCurveHidden ? FColor(0, 32, 0) : FColor::Green;
		break;
	default:
		// A bad sub-curve index was given. 
		check(false);
		break;
	}

	return ButtonColor;
}

float UDistributionFloatUniform::GetKeyIn(int32 KeyIndex)
{
	check( KeyIndex == 0 );
	return 0.f;
}

float UDistributionFloatUniform::GetKeyOut(int32 SubIndex, int32 KeyIndex)
{
	check( SubIndex == 0 || SubIndex == 1);
	check( KeyIndex == 0 );
	return (SubIndex == 0) ? Min : Max;
}

FColor UDistributionFloatUniform::GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor)
{
	// There can only be as many as two sub-curves for this distribution.
	check( SubIndex == 0 || SubIndex == 1);
	// There can be only be one key for this distribution.
	check( KeyIndex == 0 );

	FColor KeyColor;

	if( 0 == SubIndex )
	{
		KeyColor = FColor::Red;
	} 
	else
	{
		KeyColor = FColor::Green;
	}

	return KeyColor;
}

void UDistributionFloatUniform::GetInRange(float& MinIn, float& MaxIn) const
{
	MinIn = 0.f;
	MaxIn = 0.f;
}

void UDistributionFloatUniform::GetOutRange(float& MinOut, float& MaxOut) const
{
	MinOut = Min;
	MaxOut = Max;
}

EInterpCurveMode UDistributionFloatUniform::GetKeyInterpMode(int32 KeyIndex) const
{
	check( KeyIndex == 0 );
	return CIM_Constant;
}

void UDistributionFloatUniform::GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const
{
	check( SubIndex == 0 || SubIndex == 1);
	check( KeyIndex == 0 );
	ArriveTangent = 0.f;
	LeaveTangent = 0.f;
}

float UDistributionFloatUniform::EvalSub(int32 SubIndex, float InVal)
{
	check( SubIndex == 0 || SubIndex == 1);
	return (SubIndex == 0) ? Min : Max;
}

int32 UDistributionFloatUniform::CreateNewKey(float KeyIn)
{	
	return 0;
}

void UDistributionFloatUniform::DeleteKey(int32 KeyIndex)
{
	check( KeyIndex == 0 );
}

int32 UDistributionFloatUniform::SetKeyIn(int32 KeyIndex, float NewInVal)
{
	check( KeyIndex == 0 );
	return 0;
}

void UDistributionFloatUniform::SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) 
{
	check( SubIndex == 0 || SubIndex == 1);
	check( KeyIndex == 0 );

	// We ensure that we can't move the Min past the Max.
	if(SubIndex == 0)
	{	
		Min = FMath::Min<float>(NewOutVal, Max);
	}
	else
	{
		Max = FMath::Max<float>(NewOutVal, Min);
	}

	bIsDirty = true;
}

void UDistributionFloatUniform::SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) 
{
	check( KeyIndex == 0 );
}

void UDistributionFloatUniform::SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent)
{
	check( SubIndex == 0 || SubIndex == 1);
	check( KeyIndex == 0 );
}

UDistributionFloatUniformCurve::UDistributionFloatUniformCurve(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

float UDistributionFloatUniformCurve::GetValue(float F, UObject* Data, struct FRandomStream* InRandomStream) const
{
	FVector2D Val = ConstantCurve.Eval(F, FVector2D(0.f, 0.f));
	return Val.X + (Val.Y - Val.X) * DIST_GET_RANDOM_VALUE(InRandomStream);
}

//@todo.CONSOLE
ERawDistributionOperation UDistributionFloatUniformCurve::GetOperation() const
{
	if (ConstantCurve.Points.Num() == 1)
	{
		// Only a single point - so see if Min == Max
		const FInterpCurvePoint<FVector2D>& Value = ConstantCurve.Points[0];
		if (Value.OutVal.X == Value.OutVal.Y)
		{
			// This may as well be a constant - don't bother doing the FMath::SRand scaling on it.
			return RDO_None;
		}
	}
	return RDO_Random;
}

uint32 UDistributionFloatUniformCurve::InitializeRawEntry(float Time, float* Values) const
{
	FVector2D MinMax = GetMinMaxValue(Time, NULL);
	Values[0] = MinMax.X;
	Values[1] = MinMax.Y;
	return 2;
}
//#endif

FVector2D UDistributionFloatUniformCurve::GetMinMaxValue(float F, UObject* Data) const
{
	return ConstantCurve.Eval(F, FVector2D(0.f, 0.f));
}

int32 UDistributionFloatUniformCurve::GetNumKeys() const
{
	return ConstantCurve.Points.Num();
}

int32 UDistributionFloatUniformCurve::GetNumSubCurves() const
{
	return 2;
}

FColor UDistributionFloatUniformCurve::GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const
{
	// Check for array out of bounds because it will crash the program
	check(SubCurveIndex >= 0);
	check(SubCurveIndex < GetNumSubCurves());

	FColor ButtonColor;

	switch(SubCurveIndex)
	{
	case 0:
		// Red
		ButtonColor = bIsSubCurveHidden ? FColor(32, 0, 0) : FColor::Red;
		break;
	case 1:
		// Green
		ButtonColor = bIsSubCurveHidden ? FColor(0, 32, 0) : FColor::Green;
		break;
	default:
		// A bad sub-curve index was given. 
		check(false);
		break;
	}

	return ButtonColor;
}

float UDistributionFloatUniformCurve::GetKeyIn(int32 KeyIndex)
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	return ConstantCurve.Points[KeyIndex].InVal;
}

float UDistributionFloatUniformCurve::GetKeyOut(int32 SubIndex, int32 KeyIndex)
{
	check((SubIndex >= 0) && (SubIndex < 2));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	
	if (SubIndex == 0)
	{
		return ConstantCurve.Points[KeyIndex].OutVal.X;
	}
	else
	{
		return ConstantCurve.Points[KeyIndex].OutVal.Y;
	}
}

FColor UDistributionFloatUniformCurve::GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor)
{
	check((SubIndex >= 0) && (SubIndex < 2));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	if (SubIndex == 0)
	{
		return FColor::Red;
	}
	else
	{
		return FColor::Green;
	}
}

void UDistributionFloatUniformCurve::GetInRange(float& MinIn, float& MaxIn) const
{
	if (ConstantCurve.Points.Num() == 0)
	{
		MinIn = 0.f;
		MaxIn = 0.f;
	}
	else
	{
		float Min = UE_BIG_NUMBER;
		float Max = -UE_BIG_NUMBER;
		for (int32 Index = 0; Index < ConstantCurve.Points.Num(); Index++)
		{
			float Value = ConstantCurve.Points[Index].InVal;
			if (Value < Min)
			{
				Min = Value;
			}
			if (Value > Max)
			{
				Max = Value;
			}
		}
		MinIn = Min;
		MaxIn = Max;
	}
}

void UDistributionFloatUniformCurve::GetOutRange(float& MinOut, float& MaxOut) const
{
	FVector2D MinVec, MaxVec;
	ConstantCurve.CalcBounds(MinVec, MaxVec, FVector2D::ZeroVector);
	MinOut = MinVec.GetMin();
	MaxOut = MaxVec.GetMax();
}

EInterpCurveMode UDistributionFloatUniformCurve::GetKeyInterpMode(int32 KeyIndex) const
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	return ConstantCurve.Points[KeyIndex].InterpMode;
}

void UDistributionFloatUniformCurve::GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const
{
	check((SubIndex >= 0) && (SubIndex < 2));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	if (SubIndex == 0)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.X;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.X;
	}
	else
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.Y;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.Y;
	}
}

float UDistributionFloatUniformCurve::EvalSub(int32 SubIndex, float InVal)
{
	check((SubIndex >= 0) && (SubIndex < 2));

	FVector2D OutVal = ConstantCurve.Eval(InVal, FVector2D::ZeroVector);

	if (SubIndex == 0)
	{
		return OutVal.X;
	}
	else
	{
		return OutVal.Y;
	}
}

int32 UDistributionFloatUniformCurve::CreateNewKey(float KeyIn)
{	
	FVector2D NewKeyVal = ConstantCurve.Eval(KeyIn, FVector2D::ZeroVector);
	int32 NewPointIndex = ConstantCurve.AddPoint(KeyIn, NewKeyVal);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;

	return NewPointIndex;
}

void UDistributionFloatUniformCurve::DeleteKey(int32 KeyIndex)
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	ConstantCurve.Points.RemoveAt(KeyIndex);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

int32 UDistributionFloatUniformCurve::SetKeyIn(int32 KeyIndex, float NewInVal)
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	int32 NewPointIndex = ConstantCurve.MovePoint(KeyIndex, NewInVal);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;

	return NewPointIndex;
}

void UDistributionFloatUniformCurve::SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) 
{
	check((SubIndex >= 0) && (SubIndex < 2));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	if (SubIndex == 0)
	{
		ConstantCurve.Points[KeyIndex].OutVal.X = NewOutVal;
	}
	else
	{
		ConstantCurve.Points[KeyIndex].OutVal.Y = NewOutVal;
	}

	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

void UDistributionFloatUniformCurve::SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) 
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	ConstantCurve.Points[KeyIndex].InterpMode = NewMode;
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

void UDistributionFloatUniformCurve::SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent)
{
	check((SubIndex >= 0) && (SubIndex < 2));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	if(SubIndex == 0)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.X = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.X = LeaveTangent;
	}
	else
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.Y = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.Y = LeaveTangent;
	}

	bIsDirty = true;
}

UDistributionVectorConstant::UDistributionVectorConstant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UDistributionVectorConstant::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_NeedLoad) && (GetOuter()->IsA(UParticleModule::StaticClass()) || GetOuter()->IsA(USoundNode::StaticClass())))
	{
		// Set to a bogus value for distributions created before VER_UE4_MOVE_DISTRIBUITONS_TO_POSTINITPROPS
		// to be able to restore to the previous default value.
		Constant = FVector(UDistribution::DefaultValue);
	}
}

void UDistributionVectorConstant::PostLoad()
{
	Super::PostLoad();
	// Reset to default for distributions created before VER_UE4_MOVE_DISTRIBUITONS_TO_POSTINITPROPS.
	if (Constant == FVector(UDistribution::DefaultValue))
	{
		Constant = FVector::ZeroVector;
	}
}

FVector UDistributionVectorConstant::GetValue(float F, UObject* Data, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	switch (LockedAxes)
	{
	case EDVLF_XY:
		return FVector(Constant.X, Constant.X, Constant.Z);
	case EDVLF_XZ:
		return FVector(Constant.X, Constant.Y, Constant.X);
	case EDVLF_YZ:
		return FVector(Constant.X, Constant.Y, Constant.Y);
	case EDVLF_XYZ:
		return FVector(Constant.X);
	case EDVLF_None:
	default:
		return Constant;
	}
}

int32 UDistributionVectorConstant::GetNumKeys() const
{
	return 1;
}

int32 UDistributionVectorConstant::GetNumSubCurves() const
{
	switch (LockedAxes)
	{
	case EDVLF_XY:
	case EDVLF_XZ:
	case EDVLF_YZ:
		return 2;
	case EDVLF_XYZ:
		return 1;
	}
	return 3;
}

FColor UDistributionVectorConstant::GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const
{
	// Check for array out of bounds because it will crash the program
	check(SubCurveIndex >= 0);
	check(SubCurveIndex < GetNumSubCurves());

	FColor ButtonColor;

	switch(SubCurveIndex)
	{
	case 0:
		// Red
		ButtonColor = bIsSubCurveHidden ? FColor(32, 0, 0) : FColor::Red;
		break;
	case 1:
		// Green
		ButtonColor = bIsSubCurveHidden ? FColor(0, 32, 0) : FColor::Green;
		break;
	case 2:
		// Blue
		ButtonColor = bIsSubCurveHidden ? FColor(0, 0, 32) : FColor::Blue;
		break;
	default:
		// A bad sub-curve index was given. 
		check(false);
		break;
	}

	return ButtonColor;
}

float UDistributionVectorConstant::GetKeyIn(int32 KeyIndex)
{
	check( KeyIndex == 0 );
	return 0.f;
}

float UDistributionVectorConstant::GetKeyOut(int32 SubIndex, int32 KeyIndex)
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex == 0 );
	
	if (SubIndex == 0)
	{
		return Constant.X;
	}
	else 
	if (SubIndex == 1)
	{
		if ((LockedAxes == EDVLF_XY) || (LockedAxes == EDVLF_XYZ))
		{
			return Constant.X;
		}
		else
		{
			return Constant.Y;
		}
	}
	else 
	{
		if ((LockedAxes == EDVLF_XZ) || (LockedAxes == EDVLF_XYZ))
		{
			return Constant.X;
		}
		else
		if (LockedAxes == EDVLF_YZ)
		{
			return Constant.Y;
		}
		else
		{
			return Constant.Z;
		}
	}
}

FColor UDistributionVectorConstant::GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor)
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex == 0 );

	if (SubIndex == 0)
	{
		return FColor::Red;
	}
	else if (SubIndex == 1)
	{
		return FColor::Green;
	}
	else
	{
		return FColor::Blue;
	}
}

void UDistributionVectorConstant::GetInRange(float& MinIn, float& MaxIn) const
{
	MinIn = 0.f;
	MaxIn = 0.f;
}

void UDistributionVectorConstant::GetOutRange(float& MinOut, float& MaxOut) const
{
	FVector Local;

	switch (LockedAxes)
	{
	case EDVLF_XY:
		Local = FVector(Constant.X, Constant.X, Constant.Z);
		break;
	case EDVLF_XZ:
		Local = FVector(Constant.X, Constant.Y, Constant.X);
		break;
	case EDVLF_YZ:
		Local = FVector(Constant.X, Constant.Y, Constant.Y);
		break;
	case EDVLF_XYZ:
		Local = FVector(Constant.X);
		break;
	case EDVLF_None:
	default:
		Local = FVector(Constant);
		break;
	}

	MinOut = Local.GetMin();
	MaxOut = Local.GetMax();
}

EInterpCurveMode UDistributionVectorConstant::GetKeyInterpMode(int32 KeyIndex) const
{
	check( KeyIndex == 0 );
	return CIM_Constant;
}

void UDistributionVectorConstant::GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex == 0 );
	ArriveTangent = 0.f;
	LeaveTangent = 0.f;
}

float UDistributionVectorConstant::EvalSub(int32 SubIndex, float InVal)
{
	check( SubIndex >= 0 && SubIndex < 3);
	return GetKeyOut(SubIndex, 0);
}

int32 UDistributionVectorConstant::CreateNewKey(float KeyIn)
{	
	return 0;
}

void UDistributionVectorConstant::DeleteKey(int32 KeyIndex)
{
	check( KeyIndex == 0 );
}

int32 UDistributionVectorConstant::SetKeyIn(int32 KeyIndex, float NewInVal)
{
	check( KeyIndex == 0 );
	return 0;
}

void UDistributionVectorConstant::SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) 
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex == 0 );

	if(SubIndex == 0)
		Constant.X = NewOutVal;
	else if(SubIndex == 1)
		Constant.Y = NewOutVal;
	else if(SubIndex == 2)
		Constant.Z = NewOutVal;

	bIsDirty = true;
}

void UDistributionVectorConstant::SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) 
{
	check( KeyIndex == 0 );
}

void UDistributionVectorConstant::SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent)
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex == 0 );
}

void UDistributionVectorConstant::GetRange(FVector& OutMin, FVector& OutMax) const
{
	OutMin	= Constant;
	OutMax	= Constant;
}

UDistributionVectorConstantCurve::UDistributionVectorConstantCurve(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FVector UDistributionVectorConstantCurve::GetValue(float F, UObject* Data, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	FVector Val = ConstantCurve.Eval(F, FVector::ZeroVector);
	switch (LockedAxes)
	{
	case EDVLF_XY:
		return FVector(Val.X, Val.X, Val.Z);
	case EDVLF_XZ:
		return FVector(Val.X, Val.Y, Val.X);
	case EDVLF_YZ:
		return FVector(Val.X, Val.Y, Val.Y);
	case EDVLF_XYZ:
		return FVector(Val.X);
	case EDVLF_None:
	default:
		return Val;
	}
}

int32 UDistributionVectorConstantCurve::GetNumKeys() const
{
	return ConstantCurve.Points.Num();
}

int32 UDistributionVectorConstantCurve::GetNumSubCurves() const
{
	switch (LockedAxes)
	{
	case EDVLF_XY:
	case EDVLF_XZ:
	case EDVLF_YZ:
		return 2;
	case EDVLF_XYZ:
		return 1;
	}
	return 3;
}

FColor UDistributionVectorConstantCurve::GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const
{
	// Check for array out of bounds because it will crash the program
	check(SubCurveIndex >= 0);
	check(SubCurveIndex < GetNumSubCurves());

	FColor ButtonColor;

	switch(SubCurveIndex)
	{
	case 0:
		// Red
		ButtonColor = bIsSubCurveHidden ? FColor(32, 0, 0) : FColor::Red;
		break;
	case 1:
		// Green
		ButtonColor = bIsSubCurveHidden ? FColor(0, 32, 0) : FColor::Green;
		break;
	case 2:
		// Blue
		ButtonColor = bIsSubCurveHidden ? FColor(0, 0, 32) : FColor::Blue;
		break;
	default:
		// A bad sub-curve index was given. 
		check(false);
		break;
	}

	return ButtonColor;
}

float UDistributionVectorConstantCurve::GetKeyIn(int32 KeyIndex)
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	return ConstantCurve.Points[KeyIndex].InVal;
}

float UDistributionVectorConstantCurve::GetKeyOut(int32 SubIndex, int32 KeyIndex)
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	
	if (SubIndex == 0)
	{
		return ConstantCurve.Points[KeyIndex].OutVal.X;
	}
	else
	if(SubIndex == 1)
	{
		if ((LockedAxes == EDVLF_XY) || (LockedAxes == EDVLF_XYZ))
		{
			return ConstantCurve.Points[KeyIndex].OutVal.X;
		}

		return ConstantCurve.Points[KeyIndex].OutVal.Y;
	}
	else 
	{
		if ((LockedAxes == EDVLF_XZ) || (LockedAxes == EDVLF_XYZ))
		{
			return ConstantCurve.Points[KeyIndex].OutVal.X;
		}
		else
		if (LockedAxes == EDVLF_YZ)
		{
			return ConstantCurve.Points[KeyIndex].OutVal.Y;
		}

		return ConstantCurve.Points[KeyIndex].OutVal.Z;
	}
}

FColor UDistributionVectorConstantCurve::GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor)
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );

	if (SubIndex == 0)
	{
		return FColor::Red;
	}
	else if (SubIndex == 1)
	{
		return FColor::Green;
	}
	else
	{
		return FColor::Blue;
	}
}

void UDistributionVectorConstantCurve::GetInRange(float& MinIn, float& MaxIn) const
{
	if( ConstantCurve.Points.Num() == 0 )
	{
		MinIn = 0.f;
		MaxIn = 0.f;
	}
	else
	{
		float Min = UE_BIG_NUMBER;
		float Max = -UE_BIG_NUMBER;
		for (int32 Index = 0; Index < ConstantCurve.Points.Num(); Index++)
		{
			float Value = ConstantCurve.Points[Index].InVal;
			if (Value < Min)
			{
				Min = Value;
			}
			if (Value > Max)
			{
				Max = Value;
			}
		}
		MinIn = Min;
		MaxIn = Max;
	}
}

void UDistributionVectorConstantCurve::GetOutRange(float& MinOut, float& MaxOut) const
{
	FVector MinVec, MaxVec;
	ConstantCurve.CalcBounds(MinVec, MaxVec, FVector::ZeroVector);

	switch (LockedAxes)
	{
	case EDVLF_XY:
		MinVec.Y = MinVec.X;
		MaxVec.Y = MaxVec.X;
		break;
	case EDVLF_XZ:
		MinVec.Z = MinVec.X;
		MaxVec.Z = MaxVec.X;
		break;
	case EDVLF_YZ:
		MinVec.Z = MinVec.Y;
		MaxVec.Z = MaxVec.Y;
		break;
	case EDVLF_XYZ:
		MinVec.Y = MinVec.X;
		MinVec.Z = MinVec.X;
		MaxVec.Y = MaxVec.X;
		MaxVec.Z = MaxVec.X;
		break;
	case EDVLF_None:
	default:
		break;
	}

	MinOut = MinVec.GetMin();
	MaxOut = MaxVec.GetMax();
}

EInterpCurveMode UDistributionVectorConstantCurve::GetKeyInterpMode(int32 KeyIndex) const
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	return ConstantCurve.Points[KeyIndex].InterpMode;
}

void UDistributionVectorConstantCurve::GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );

	if(SubIndex == 0)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.X;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.X;
	}
	else if(SubIndex == 1)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.Y;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.Y;
	}
	else if(SubIndex == 2)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.Z;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.Z;
	}
}

float UDistributionVectorConstantCurve::EvalSub(int32 SubIndex, float InVal)
{
	check( SubIndex >= 0 && SubIndex < 3);

	FVector OutVal = ConstantCurve.Eval(InVal, FVector::ZeroVector);

	if(SubIndex == 0)
		return OutVal.X;
	else if(SubIndex == 1)
		return OutVal.Y;
	else
		return OutVal.Z;
}

int32 UDistributionVectorConstantCurve::CreateNewKey(float KeyIn)
{	
	FVector NewKeyVal = ConstantCurve.Eval(KeyIn, FVector::ZeroVector);
	int32 NewPointIndex = ConstantCurve.AddPoint(KeyIn, NewKeyVal);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;

	return NewPointIndex;
}

void UDistributionVectorConstantCurve::DeleteKey(int32 KeyIndex)
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	ConstantCurve.Points.RemoveAt(KeyIndex);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

int32 UDistributionVectorConstantCurve::SetKeyIn(int32 KeyIndex, float NewInVal)
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	int32 NewPointIndex = ConstantCurve.MovePoint(KeyIndex, NewInVal);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;

	return NewPointIndex;
}

void UDistributionVectorConstantCurve::SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) 
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );

	if(SubIndex == 0)
		ConstantCurve.Points[KeyIndex].OutVal.X = NewOutVal;
	else if(SubIndex == 1)
		ConstantCurve.Points[KeyIndex].OutVal.Y = NewOutVal;
	else 
		ConstantCurve.Points[KeyIndex].OutVal.Z = NewOutVal;

	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

void UDistributionVectorConstantCurve::SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) 
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	
	ConstantCurve.Points[KeyIndex].InterpMode = NewMode;
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

void UDistributionVectorConstantCurve::SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent)
{
	check( SubIndex >= 0 && SubIndex < 3);
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );

	if(SubIndex == 0)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.X = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.X = LeaveTangent;
	}
	else if(SubIndex == 1)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.Y = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.Y = LeaveTangent;
	}
	else if(SubIndex == 2)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.Z = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.Z = LeaveTangent;
	}

	bIsDirty = true;
}

// DistributionVector interface
void UDistributionVectorConstantCurve::GetRange(FVector& OutMin, FVector& OutMax) const
{
	FVector MinVec, MaxVec;
	ConstantCurve.CalcBounds(MinVec, MaxVec, FVector::ZeroVector);

	switch (LockedAxes)
	{
	case EDVLF_XY:
		MinVec.Y = MinVec.X;
		MaxVec.Y = MaxVec.X;
		break;
	case EDVLF_XZ:
		MinVec.Z = MinVec.X;
		MaxVec.Z = MaxVec.X;
		break;
	case EDVLF_YZ:
		MinVec.Z = MinVec.Y;
		MaxVec.Z = MaxVec.Y;
		break;
	case EDVLF_XYZ:
		MinVec.Y = MinVec.X;
		MinVec.Z = MinVec.X;
		MaxVec.Y = MaxVec.X;
		MaxVec.Z = MaxVec.X;
		break;
	case EDVLF_None:
	default:
		break;
	}

	OutMin = MinVec;
	OutMax = MaxVec;
}

UDistributionVectorUniform::UDistributionVectorUniform(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MirrorFlags[0] = EDVMF_Different;
	MirrorFlags[1] = EDVMF_Different;
	MirrorFlags[2] = EDVMF_Different;
	bUseExtremes = false;
}

void UDistributionVectorUniform::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_NeedLoad) && (GetOuter()->IsA(UParticleModule::StaticClass()) || GetOuter()->IsA(USoundNode::StaticClass())))
	{
		// Set to a bogus value for distributions created before VER_UE4_MOVE_DISTRIBUITONS_TO_POSTINITPROPS
		// to be able to restore to the previous default value.
		Min = FVector(UDistribution::DefaultValue);
		Max = FVector(UDistribution::DefaultValue);
	}
}

void UDistributionVectorUniform::PostLoad()
{
	Super::PostLoad();
	// Reset to default for distributions created before VER_UE4_MOVE_DISTRIBUITONS_TO_POSTINITPROPS.
	if (Min == FVector(UDistribution::DefaultValue))
	{
		Min = FVector::ZeroVector;
	}
	if (Max == FVector(UDistribution::DefaultValue))
	{
		Max = FVector::ZeroVector;
	}
}

FVector UDistributionVectorUniform::GetValue(float F, UObject* Data, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	FVector LocalMax = Max;
	FVector LocalMin = Min;

	LocalMin.X = (MirrorFlags[0] == EDVMF_Different) ? LocalMin.X : ((MirrorFlags[0] == EDVMF_Mirror) ? -LocalMax.X : LocalMax.X);
	LocalMin.Y = (MirrorFlags[1] == EDVMF_Different) ? LocalMin.Y : ((MirrorFlags[1] == EDVMF_Mirror) ? -LocalMax.Y : LocalMax.Y);
	LocalMin.Z = (MirrorFlags[2] == EDVMF_Different) ? LocalMin.Z : ((MirrorFlags[2] == EDVMF_Mirror) ? -LocalMax.Z : LocalMax.Z);

	float fX;
	float fY;
	float fZ;

	bool bMin = true;
	if (bUseExtremes)
	{
		if (Extreme == 0)
		{
			if (DIST_GET_RANDOM_VALUE(InRandomStream) > 0.5f)
			{
				bMin = false;
			}
		}
		else if (Extreme > 0)
		{
			bMin = false;
		}
	}

	switch (LockedAxes)
	{
	case EDVLF_XY:
		if (bUseExtremes)
		{
			if (bMin)
			{
				fX = LocalMin.X;
				fZ = LocalMin.Z;
			}
			else
			{
				fX = LocalMax.X;
				fZ = LocalMax.Z;
			}
		}
		else
		{
			fX = LocalMax.X + (LocalMin.X - LocalMax.X) * DIST_GET_RANDOM_VALUE(InRandomStream);
			fZ = LocalMax.Z + (LocalMin.Z - LocalMax.Z) * DIST_GET_RANDOM_VALUE(InRandomStream);
		}
		fY = fX;
		break;
	case EDVLF_XZ:
		if (bUseExtremes)
		{
			if (bMin)
			{
				fX = LocalMin.X;
				fY = LocalMin.Y;
			}
			else
			{
				fX = LocalMax.X;
				fY = LocalMax.Y;
			}
		}
		else
		{
			fX = LocalMax.X + (LocalMin.X - LocalMax.X) * DIST_GET_RANDOM_VALUE(InRandomStream);
			fY = LocalMax.Y + (LocalMin.Y - LocalMax.Y) * DIST_GET_RANDOM_VALUE(InRandomStream);
		}
		fZ = fX;
		break;
	case EDVLF_YZ:
		if (bUseExtremes)
		{
			if (bMin)
			{
				fX = LocalMin.X;
				fY = LocalMin.Y;
			}
			else
			{
				fX = LocalMax.X;
				fY = LocalMax.Y;
			}
		}
		else
		{
			fX = LocalMax.X + (LocalMin.X - LocalMax.X) * DIST_GET_RANDOM_VALUE(InRandomStream);
			fY = LocalMax.Y + (LocalMin.Y - LocalMax.Y) * DIST_GET_RANDOM_VALUE(InRandomStream);
		}
		fZ = fY;
		break;
	case EDVLF_XYZ:
		if (bUseExtremes)
		{
			if (bMin)
			{
				fX = LocalMin.X;
			}
			else
			{
				fX = LocalMax.X;
			}
		}
		else
		{
			fX = LocalMax.X + (LocalMin.X - LocalMax.X) * DIST_GET_RANDOM_VALUE(InRandomStream);
		}
		fY = fX;
		fZ = fX;
		break;
	case EDVLF_None:
	default:
		if (bUseExtremes)
		{
			if (bMin)
			{
				fX = LocalMin.X;
				fY = LocalMin.Y;
				fZ = LocalMin.Z;
			}
			else
			{
				fX = LocalMax.X;
				fY = LocalMax.Y;
				fZ = LocalMax.Z;
			}
		}
		else
		{
			fX = LocalMax.X + (LocalMin.X - LocalMax.X) * DIST_GET_RANDOM_VALUE(InRandomStream);
			fY = LocalMax.Y + (LocalMin.Y - LocalMax.Y) * DIST_GET_RANDOM_VALUE(InRandomStream);
			fZ = LocalMax.Z + (LocalMin.Z - LocalMax.Z) * DIST_GET_RANDOM_VALUE(InRandomStream);
		}
		break;
	}

	return FVector(fX, fY, fZ);
}

//@todo.CONSOLE
ERawDistributionOperation UDistributionVectorUniform::GetOperation() const
{
	if (Min == Max)
	{
		// This may as well be a constant - don't bother doing the FMath::SRand scaling on it.
		return RDO_None;
	}
	// override the operation to use
	return bUseExtremes ? RDO_Extreme : RDO_Random;
}

uint8 UDistributionVectorUniform::GetLockFlag() const
{
	return (uint8)LockedAxes;
}

uint32 UDistributionVectorUniform::InitializeRawEntry(float Time, float* Values) const
{
	// get the locked/mirrored min and max
	FVector ValueMin = GetMinValue();
	FVector ValueMax = GetMaxValue();
	Values[0] = ValueMin.X;
	Values[1] = ValueMin.Y;
	Values[2] = ValueMin.Z;
	Values[3] = ValueMax.X;
	Values[4] = ValueMax.Y;
	Values[5] = ValueMax.Z;

	// six elements per value
	return 6;
}

//#endif

FVector UDistributionVectorUniform::GetMinValue() const
{
	FVector LocalMax = Max;
	FVector LocalMin = Min;

	for (int32 i = 0; i < 3; i++)
	{
		switch (MirrorFlags[i])
		{
		case EDVMF_Same:	LocalMin[i] =  LocalMax[i];		break;
		case EDVMF_Mirror:	LocalMin[i] = -LocalMax[i];		break;
		}
	}

	float fX;
	float fY;
	float fZ;

	switch (LockedAxes)
	{
	case EDVLF_XY:
		fX = LocalMin.X;
		fY = LocalMin.X;
		fZ = LocalMin.Z;
		break;
	case EDVLF_XZ:
		fX = LocalMin.X;
		fY = LocalMin.Y;
		fZ = fX;
		break;
	case EDVLF_YZ:
		fX = LocalMin.X;
		fY = LocalMin.Y;
		fZ = fY;
		break;
	case EDVLF_XYZ:
		fX = LocalMin.X;
		fY = fX;
		fZ = fX;
		break;
	case EDVLF_None:
	default:
		fX = LocalMin.X;
		fY = LocalMin.Y;
		fZ = LocalMin.Z;
		break;
	}

	return FVector(fX, fY, fZ);
}

FVector UDistributionVectorUniform::GetMaxValue() const
{
	FVector LocalMax = Max;

	float fX;
	float fY;
	float fZ;

	switch (LockedAxes)
	{
	case EDVLF_XY:
		fX = LocalMax.X;
		fY = LocalMax.X;
		fZ = LocalMax.Z;
		break;
	case EDVLF_XZ:
		fX = LocalMax.X;
		fY = LocalMax.Y;
		fZ = fX;
		break;
	case EDVLF_YZ:
		fX = LocalMax.X;
		fY = LocalMax.Y;
		fZ = fY;
		break;
	case EDVLF_XYZ:
		fX = LocalMax.X;
		fY = fX;
		fZ = fX;
		break;
	case EDVLF_None:
	default:
		fX = LocalMax.X;
		fY = LocalMax.Y;
		fZ = LocalMax.Z;
		break;
	}

	return FVector(fX, fY, fZ);
}

int32 UDistributionVectorUniform::GetNumKeys() const
{
	return 1;
}

int32 UDistributionVectorUniform::GetNumSubCurves() const
{
	switch (LockedAxes)
	{
	case EDVLF_XY:
	case EDVLF_XZ:
	case EDVLF_YZ:
		return 4;
	case EDVLF_XYZ:
		return 2;
	}
	return 6;
}


FColor UDistributionVectorUniform::GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const
{
	const int32 SubCurves = GetNumSubCurves();

	// Check for array out of bounds because it will crash the program
	check(SubCurveIndex >= 0);
	check(SubCurveIndex < SubCurves);

	const bool bShouldGroupMinAndMax = ( (SubCurves == 4) || (SubCurves == 6) );
	FColor ButtonColor;

	switch(SubCurveIndex)
	{
	case 0:
		// Red
		ButtonColor = bIsSubCurveHidden ? FColor(32, 0, 0) : FColor::Red;
		break;
	case 1:
		if (bShouldGroupMinAndMax)
		{
			// Dark red
			ButtonColor = bIsSubCurveHidden ? FColor(28, 0, 0) : FColor(196, 0, 0);
		}
		else
		{
			// Green
			ButtonColor = bIsSubCurveHidden ? FColor(0, 32,  0) : FColor::Green;
		}
		break;
	case 2:
		if (bShouldGroupMinAndMax)
		{
			// Green
			ButtonColor = bIsSubCurveHidden ? FColor(0, 32, 0) : FColor::Green;
		}
		else
		{
			// Blue
			ButtonColor = bIsSubCurveHidden ? FColor(0, 0, 32) : FColor::Blue;
		}
		break;
	case 3:
		// Dark green
		ButtonColor = bIsSubCurveHidden ? FColor(0, 28, 0) : FColor(0, 196, 0);
		break;
	case 4:
		// Blue
		ButtonColor = bIsSubCurveHidden ? FColor(0, 0, 32) : FColor::Blue;
		break;
	case 5:
		// Dark blue
		ButtonColor = bIsSubCurveHidden ? FColor(0, 0, 28) : FColor(0, 0, 196);
		break;
	default:
		// A bad sub-curve index was given. 
		check(false);
		break;
	}

	return ButtonColor;
}

float UDistributionVectorUniform::GetKeyIn(int32 KeyIndex)
{
	check( KeyIndex == 0 );
	return 0.f;
}

float UDistributionVectorUniform::GetKeyOut(int32 SubIndex, int32 KeyIndex)
{
	check( SubIndex >= 0 && SubIndex < 6 );
	check( KeyIndex == 0 );

	FVector LocalMax = Max;
	FVector LocalMin = Min;

	for (int32 i = 0; i < 3; i++)
	{
		switch (MirrorFlags[i])
		{
		case EDVMF_Same:	LocalMin[i] =  LocalMax[i];		break;
		case EDVMF_Mirror:	LocalMin[i] = -LocalMax[i];		break;
		}
	}

	switch (LockedAxes)
	{
	case EDVLF_XY:
		LocalMin.Y = LocalMin.X;
		break;
	case EDVLF_XZ:
		LocalMin.Z = LocalMin.X;
		break;
	case EDVLF_YZ:
		LocalMin.Z = LocalMin.Y;
		break;
	case EDVLF_XYZ:
		LocalMin.Y = LocalMin.X;
		LocalMin.Z = LocalMin.X;
		break;
	case EDVLF_None:
	default:
		break;
	}

	switch (SubIndex)
	{
	case 0:		return LocalMin.X;
	case 1:		return LocalMax.X;
	case 2:		return LocalMin.Y;
	case 3:		return LocalMax.Y;
	case 4:		return LocalMin.Z;
	}
	return LocalMax.Z;
}

FColor UDistributionVectorUniform::GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor)
{
	check( SubIndex >= 0 && SubIndex < 6);
	check( KeyIndex == 0 );

	if (SubIndex == 0)
	{
		return FColor(128, 0, 0);
	}
	else if (SubIndex == 1)
	{
		return FColor::Red;
	}
	else if (SubIndex == 2)
	{
		return FColor(0, 128, 0);
	}
	else if (SubIndex == 3)
	{
		return FColor::Green;
	}
	else if (SubIndex == 4)
	{
		return FColor(0, 0, 128);
	}
	else
	{
		return FColor::Blue;
	}
}

void UDistributionVectorUniform::GetInRange(float& MinIn, float& MaxIn) const
{
	MinIn = 0.f;
	MaxIn = 0.f;
}

void UDistributionVectorUniform::GetOutRange(float& MinOut, float& MaxOut) const
{
	FVector LocalMax = Max;
	FVector LocalMin = Min;

	for (int32 i = 0; i < 3; i++)
	{
		switch (MirrorFlags[i])
		{
		case EDVMF_Same:	LocalMin[i] =  LocalMax[i];		break;
		case EDVMF_Mirror:	LocalMin[i] = -LocalMax[i];		break;
		}
	}

	FVector LocalMin2;
	FVector LocalMax2;

	switch (LockedAxes)
	{
	case EDVLF_XY:
		LocalMin2 = FVector(LocalMin.X, LocalMin.X, LocalMin.Z);
		LocalMax2 = FVector(LocalMax.X, LocalMax.X, LocalMax.Z);
		break;
	case EDVLF_XZ:
		LocalMin2 = FVector(LocalMin.X, LocalMin.Y, LocalMin.X);
		LocalMax2 = FVector(LocalMax.X, LocalMax.Y, LocalMax.X);
		break;
	case EDVLF_YZ:
		LocalMin2 = FVector(LocalMin.X, LocalMin.Y, LocalMin.Y);
		LocalMax2 = FVector(LocalMax.X, LocalMax.Y, LocalMax.Y);
		break;
	case EDVLF_XYZ:
		LocalMin2 = FVector(LocalMin.X);
		LocalMax2 = FVector(LocalMax.X);
		break;
	case EDVLF_None:
	default:
		LocalMin2 = FVector(LocalMin.X, LocalMin.Y, LocalMin.Z);
		LocalMax2 = FVector(LocalMax.X, LocalMax.Y, LocalMax.Z);
		break;
	}

	MinOut = LocalMin2.GetMin();
	MaxOut = LocalMax2.GetMax();
}

EInterpCurveMode UDistributionVectorUniform::GetKeyInterpMode(int32 KeyIndex) const
{
	check( KeyIndex == 0 );
	return CIM_Constant;
}

void UDistributionVectorUniform::GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const
{
	check( SubIndex >= 0 && SubIndex < 6);
	check( KeyIndex == 0 );
	ArriveTangent = 0.f;
	LeaveTangent = 0.f;
}

float UDistributionVectorUniform::EvalSub(int32 SubIndex, float InVal)
{
	return GetKeyOut(SubIndex, 0);
}

int32 UDistributionVectorUniform::CreateNewKey(float KeyIn)
{	
	return 0;
}

void UDistributionVectorUniform::DeleteKey(int32 KeyIndex)
{
	check( KeyIndex == 0 );
}

int32 UDistributionVectorUniform::SetKeyIn(int32 KeyIndex, float NewInVal)
{
	check( KeyIndex == 0 );
	return 0;
}

void UDistributionVectorUniform::SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) 
{
	check( SubIndex >= 0 && SubIndex < 6 );
	check( KeyIndex == 0 );

	if(SubIndex == 0)
		Min.X = FMath::Min<float>(NewOutVal, Max.X);
	else if(SubIndex == 1)
		Max.X = FMath::Max<float>(NewOutVal, Min.X);
	else if(SubIndex == 2)
		Min.Y = FMath::Min<float>(NewOutVal, Max.Y);
	else if(SubIndex == 3)
		Max.Y = FMath::Max<float>(NewOutVal, Min.Y);
	else if(SubIndex == 4)
		Min.Z = FMath::Min<float>(NewOutVal, Max.Z);
	else
		Max.Z = FMath::Max<float>(NewOutVal, Min.Z);

	bIsDirty = true;
}

void UDistributionVectorUniform::SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) 
{
	check( KeyIndex == 0 );
}

void UDistributionVectorUniform::SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent)
{
	check( SubIndex >= 0 && SubIndex < 6);
	check( KeyIndex == 0 );
}

// DistributionVector interface
void UDistributionVectorUniform::GetRange(FVector& OutMin, FVector& OutMax) const
{
	OutMin	= Min;
	OutMax	= Max;
}


UDistributionVectorUniformCurve::UDistributionVectorUniformCurve(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bLockAxes1 = false;
	bLockAxes2 = false;
	LockedAxes[0] = EDVLF_None;
	LockedAxes[1] = EDVLF_None;
	MirrorFlags[0] = EDVMF_Different;
	MirrorFlags[1] = EDVMF_Different;
	MirrorFlags[2] = EDVMF_Different;
	bUseExtremes = false;
}

FVector UDistributionVectorUniformCurve::GetValue(float F, UObject* Data, int32 Extreme, struct FRandomStream* InRandomStream) const
{
	FTwoVectors	Val = ConstantCurve.Eval(F, FTwoVectors());

	bool bMin = true;
	if (bUseExtremes)
	{
		if (Extreme == 0)
		{
			if (DIST_GET_RANDOM_VALUE(InRandomStream) > 0.5f)
			{
				bMin = false;
			}
		}
		else if (Extreme < 0)
		{
			bMin = false;
		}
	}

	LockAndMirror(Val);
	if (bUseExtremes)
	{
		return ((bMin == true) ? FVector(Val.v2.X, Val.v2.Y, Val.v2.Z) : FVector(Val.v1.X, Val.v1.Y, Val.v1.Z));
	}
	else
	{
		return FVector(
			Val.v1.X + (Val.v2.X - Val.v1.X) * DIST_GET_RANDOM_VALUE(InRandomStream),
			Val.v1.Y + (Val.v2.Y - Val.v1.Y) * DIST_GET_RANDOM_VALUE(InRandomStream),
			Val.v1.Z + (Val.v2.Z - Val.v1.Z) * DIST_GET_RANDOM_VALUE(InRandomStream));
	}
}

//@todo.CONSOLE
ERawDistributionOperation UDistributionVectorUniformCurve::GetOperation() const
{
	if (ConstantCurve.Points.Num() == 1)
	{
		// Only a single point - so see if Min == Max
		const FInterpCurvePoint<FTwoVectors>& Value = ConstantCurve.Points[0];
		if (Value.OutVal.v1 == Value.OutVal.v2)
		{
			// This may as well be a constant - don't bother doing the FMath::SRand scaling on it.
			return RDO_None;
		}
	}
	return bUseExtremes ? RDO_Extreme : RDO_Random;
}


uint32 UDistributionVectorUniformCurve::InitializeRawEntry(float Time, float* Values) const
{
	// get the min and max values at the current time (just Eval the curve)
	FTwoVectors MinMax = GetMinMaxValue(Time, NULL);
	// apply any axis locks and mirroring (in place)
	LockAndMirror(MinMax);

	// copy out the values
	Values[0] = MinMax.v1.X;
	Values[1] = MinMax.v1.Y;
	Values[2] = MinMax.v1.Z;
	Values[3] = MinMax.v2.X;
	Values[4] = MinMax.v2.Y;
	Values[5] = MinMax.v2.Z;

	// we wrote size elements
	return 6;
}

//#endif


FTwoVectors UDistributionVectorUniformCurve::GetMinMaxValue(float F, UObject* Data) const
{
	return ConstantCurve.Eval(F, FTwoVectors());
}

FVector UDistributionVectorUniformCurve::GetMinValue() const
{
	check(!TEXT("Don't call me!"));
	return FVector::ZeroVector;
}

// PVS-Studio notices that the implementation of GetMinValue is identical to this one
// and warns us. In this case, it is intentional, so we disable the warning:
FVector UDistributionVectorUniformCurve::GetMaxValue() const
{
	check(!TEXT("Don't call me!"));
	return FVector::ZeroVector;
}

int32 UDistributionVectorUniformCurve::GetNumKeys() const
{
	return ConstantCurve.Points.Num();
}

int32 UDistributionVectorUniformCurve::GetNumSubCurves() const
{
	int32 Count = 0;
/***
	switch (LockedAxes[0])
	{
	case EDVLF_XY:	Count += 2;	break;
	case EDVLF_XZ:	Count += 2;	break;
	case EDVLF_YZ:	Count += 2;	break;
	case EDVLF_XYZ:	Count += 1;	break;
	default:		Count += 3;	break;
	}

	switch (LockedAxes[1])
	{
	case EDVLF_XY:	Count += 2;	break;
	case EDVLF_XZ:	Count += 2;	break;
	case EDVLF_YZ:	Count += 2;	break;
	case EDVLF_XYZ:	Count += 1;	break;
	default:		Count += 3;	break;
	}
***/
	Count = 6;
	return Count;
}

FColor UDistributionVectorUniformCurve::GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const
{
	const int32 SubCurves = GetNumSubCurves();

	// Check for array out of bounds because it will crash the program
	check(SubCurveIndex >= 0);
	check(SubCurveIndex < SubCurves);

	const bool bShouldGroupMinAndMax = ( (SubCurves == 4) || (SubCurves == 6) );
	FColor ButtonColor;

	switch(SubCurveIndex)
	{
	case 0:
		// Red
		ButtonColor = bIsSubCurveHidden ? FColor(32, 0, 0) : FColor::Red;
		break;
	case 1:
		if (bShouldGroupMinAndMax)
		{
			// Dark red
			ButtonColor = bIsSubCurveHidden ? FColor(28, 0, 0) : FColor(196, 0, 0);
		}
		else
		{
			// Green
			ButtonColor = bIsSubCurveHidden ? FColor(0, 32, 0) : FColor::Green;
		}
		break;
	case 2:
		if (bShouldGroupMinAndMax)
		{
			// Green
			ButtonColor = bIsSubCurveHidden ? FColor(0, 32, 0) : FColor::Green;
		}
		else
		{
			// Blue
			ButtonColor = bIsSubCurveHidden ? FColor(0, 0, 32) : FColor::Blue;
		}
		break;
	case 3:
		// Dark green
		ButtonColor = bIsSubCurveHidden ? FColor(  0, 28,  0) : FColor(0 , 196, 0);
		break;
	case 4:
		// Blue
		ButtonColor = bIsSubCurveHidden ? FColor(0, 0, 32) : FColor::Blue;
		break;
	case 5:
		// Dark blue
		ButtonColor = bIsSubCurveHidden ? FColor(  0,  0, 28) : FColor(0, 0, 196);
		break;
	default:
		// A bad sub-curve index was given. 
		check(false);
		break;
	}

	return ButtonColor;
}

float UDistributionVectorUniformCurve::GetKeyIn(int32 KeyIndex)
{
	check( KeyIndex >= 0 && KeyIndex < ConstantCurve.Points.Num() );
	return ConstantCurve.Points[KeyIndex].InVal;
}

float UDistributionVectorUniformCurve::GetKeyOut(int32 SubIndex, int32 KeyIndex)
{
	check((SubIndex >= 0) && (SubIndex < 6));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));


	// Grab the value
	FInterpCurvePoint<FTwoVectors>	Point = ConstantCurve.Points[KeyIndex];

	FTwoVectors	Val	= Point.OutVal;
	LockAndMirror(Val);
	if ((SubIndex % 2) == 0)
	{
		return Val.v1[SubIndex / 2];
	}
	return Val.v2[SubIndex / 2];
}

FColor UDistributionVectorUniformCurve::GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor)
{
	check((SubIndex >= 0) && (SubIndex < 6));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	if (SubIndex == 0)
	{
		return FColor::Red;
	}
	else if (SubIndex == 1)
	{
		return FColor(128,0,0);
	}
	else if (SubIndex == 2)
	{
		return FColor::Green;
	}
	else if (SubIndex == 3)
	{
		return FColor(0,128,0);
	}
	else if (SubIndex == 4)
	{
		return FColor::Blue;
	}
	else
	{
		return FColor(0,0,128);
	}
}

void UDistributionVectorUniformCurve::GetInRange(float& MinIn, float& MaxIn) const
{
	if (ConstantCurve.Points.Num() == 0)
	{
		MinIn = 0.f;
		MaxIn = 0.f;
	}
	else
	{
		float Min = UE_BIG_NUMBER;
		float Max = -UE_BIG_NUMBER;
		for (int32 Index = 0; Index < ConstantCurve.Points.Num(); Index++)
		{
			float Value = ConstantCurve.Points[Index].InVal;
			if (Value < Min)
			{
				Min = Value;
			}
			if (Value > Max)
			{
				Max = Value;
			}
		}
		MinIn = Min;
		MaxIn = Max;
	}
}

void UDistributionVectorUniformCurve::GetOutRange(float& MinOut, float& MaxOut) const
{
	FTwoVectors	MinVec, MaxVec;

	ConstantCurve.CalcBounds(MinVec, MaxVec, FTwoVectors());
	LockAndMirror(MinVec);
	LockAndMirror(MaxVec);

	MinOut = FMath::Min<float>(MinVec.GetMin(), MaxVec.GetMin());
	MaxOut = FMath::Max<float>(MinVec.GetMax(), MaxVec.GetMax());
}

EInterpCurveMode UDistributionVectorUniformCurve::GetKeyInterpMode(int32 KeyIndex) const
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	return ConstantCurve.Points[KeyIndex].InterpMode;
}

void UDistributionVectorUniformCurve::GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const
{
	check((SubIndex >= 0) && (SubIndex < 6));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	if (SubIndex == 0)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.v1.X;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.v1.X;
	}
	else
	if (SubIndex == 1)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.v2.X;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.v2.X;
	}
	else
	if (SubIndex == 2)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.v1.Y;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.v1.Y;
	}
	else
	if (SubIndex == 3)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.v2.Y;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.v2.Y;
	}
	else
	if (SubIndex == 4)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.v1.Z;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.v1.Z;
	}
	else
	if (SubIndex == 5)
	{
		ArriveTangent = ConstantCurve.Points[KeyIndex].ArriveTangent.v2.Z;
		LeaveTangent = ConstantCurve.Points[KeyIndex].LeaveTangent.v2.Z;
	}
}

float UDistributionVectorUniformCurve::EvalSub(int32 SubIndex, float InVal)
{
	check((SubIndex >= 0) && (SubIndex < 6));

	FTwoVectors Default;
	FTwoVectors OutVal = ConstantCurve.Eval(InVal, Default);
	LockAndMirror(OutVal);
	if ((SubIndex % 2) == 0)
	{
		return OutVal.v1[SubIndex / 2];
	}
	else
	{
		return OutVal.v2[SubIndex / 2];
	}
}

int32 UDistributionVectorUniformCurve::CreateNewKey(float KeyIn)
{
	FTwoVectors NewKeyVal = ConstantCurve.Eval(KeyIn, FTwoVectors());
	int32 NewPointIndex = ConstantCurve.AddPoint(KeyIn, NewKeyVal);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;

	return NewPointIndex;
}

void UDistributionVectorUniformCurve::DeleteKey(int32 KeyIndex)
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	ConstantCurve.Points.RemoveAt(KeyIndex);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

int32 UDistributionVectorUniformCurve::SetKeyIn(int32 KeyIndex, float NewInVal)
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));
	int32 NewPointIndex = ConstantCurve.MovePoint(KeyIndex, NewInVal);
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;

	return NewPointIndex;
}

void UDistributionVectorUniformCurve::SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal)
{
	check((SubIndex >= 0) && (SubIndex < 6));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	float Value;

	FInterpCurvePoint<FTwoVectors>*	Point = &(ConstantCurve.Points[KeyIndex]);
	check(Point);

	if (SubIndex == 0)
	{
		Value = FMath::Max<float>(NewOutVal, Point->OutVal.v2.X);
		Point->OutVal.v1.X = Value;
	}
	else
	if (SubIndex == 1)
	{
		Value = FMath::Min<float>(NewOutVal, Point->OutVal.v1.X);
		Point->OutVal.v2.X = Value;
	}
	else
	if (SubIndex == 2)
	{
		Value = FMath::Max<float>(NewOutVal, Point->OutVal.v2.Y);
		Point->OutVal.v1.Y = Value;
	}
	else
	if (SubIndex == 3)
	{
		Value = FMath::Min<float>(NewOutVal, Point->OutVal.v1.Y);
		Point->OutVal.v2.Y = Value;
	}
	else
	if (SubIndex == 4)
	{
		Value = FMath::Max<float>(NewOutVal, Point->OutVal.v2.Z);
		Point->OutVal.v1.Z = Value;
	}
	else
	if (SubIndex == 5)
	{
		Value = FMath::Min<float>(NewOutVal, Point->OutVal.v1.Z);
		Point->OutVal.v2.Z = Value;
	}

	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

void UDistributionVectorUniformCurve::SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode)
{
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	ConstantCurve.Points[KeyIndex].InterpMode = NewMode;
	ConstantCurve.AutoSetTangents(0.f);

	bIsDirty = true;
}

void UDistributionVectorUniformCurve::SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent)
{
	check((SubIndex >= 0) && (SubIndex < 6));
	check((KeyIndex >= 0) && (KeyIndex < ConstantCurve.Points.Num()));

	if (SubIndex == 0)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.v1.X = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.v1.X = LeaveTangent;
	}
	else
	if (SubIndex == 1)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.v2.X = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.v2.X = LeaveTangent;
	}
	else
	if (SubIndex == 2)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.v1.Y = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.v1.Y = LeaveTangent;
	}
	else
	if (SubIndex == 3)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.v2.Y = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.v2.Y = LeaveTangent;
	}
	else
	if (SubIndex == 4)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.v1.Z = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.v1.Z = LeaveTangent;
	}
	else
	if (SubIndex == 5)
	{
		ConstantCurve.Points[KeyIndex].ArriveTangent.v2.Z = ArriveTangent;
		ConstantCurve.Points[KeyIndex].LeaveTangent.v2.Z = LeaveTangent;
	}

	bIsDirty = true;
}



void UDistributionVectorUniformCurve::LockAndMirror(FTwoVectors& Val) const
{
	// Handle the mirror flags...
	for (int32 i = 0; i < 3; i++)
	{
		switch (MirrorFlags[i])
		{
		case EDVMF_Same:		Val.v2[i]	=  Val.v1[i];	break;
		case EDVMF_Mirror:		Val.v2[i]	= -Val.v1[i];	break;
		}
	}

	// Handle the lock axes flags
	switch (LockedAxes[0])
	{
	case EDVLF_XY:
		Val.v1.Y	= Val.v1.X;
		break;
	case EDVLF_XZ:
		Val.v1.Z	= Val.v1.X;
		break;
	case EDVLF_YZ:
		Val.v1.Z	= Val.v1.Y;
		break;
	case EDVLF_XYZ:
		Val.v1.Y	= Val.v1.X;
		Val.v1.Z	= Val.v1.X;
		break;
	}

	switch (LockedAxes[0])
	{
	case EDVLF_XY:
		Val.v2.Y	= Val.v2.X;
		break;
	case EDVLF_XZ:
		Val.v2.Z	= Val.v2.X;
		break;
	case EDVLF_YZ:
		Val.v2.Z	= Val.v2.Y;
		break;
	case EDVLF_XYZ:
		Val.v2.Y	= Val.v2.X;
		Val.v2.Z	= Val.v2.X;
		break;
	}
}

// DistributionVector interface
void UDistributionVectorUniformCurve::GetRange(FVector& OutMin, FVector& OutMax) const
{
	FTwoVectors	MinVec, MaxVec;

	ConstantCurve.CalcBounds(MinVec, MaxVec, FTwoVectors());
	LockAndMirror(MinVec);
	LockAndMirror(MaxVec);

	if (MinVec.v1.X < MaxVec.v1.X)	OutMin.X = MinVec.v1.X;
	else							OutMin.X = MaxVec.v1.X;
	if (MinVec.v1.Y < MaxVec.v1.Y)	OutMin.Y = MinVec.v1.Y;
	else							OutMin.Y = MaxVec.v1.Y;
	if (MinVec.v1.Z < MaxVec.v1.Z)	OutMin.Z = MinVec.v1.Z;
	else							OutMin.Z = MaxVec.v1.Z;

	if (MinVec.v2.X > MaxVec.v2.X)	OutMax.X = MinVec.v2.X;
	else							OutMax.X = MaxVec.v2.X;
	if (MinVec.v2.Y > MaxVec.v2.Y)	OutMax.Y = MinVec.v2.Y;
	else							OutMax.Y = MaxVec.v2.Y;
	if (MinVec.v2.Z > MaxVec.v2.Z)	OutMax.Z = MinVec.v2.Z;
	else							OutMax.Z = MaxVec.v2.Z;
}


UDistributionFloatParameterBase::UDistributionFloatParameterBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MaxInput = 1.0f;
	MaxOutput = 1.0f;
}

float UDistributionFloatParameterBase::GetValue( float F, UObject* Data, struct FRandomStream* InRandomStream ) const
{
	float ParamFloat = 0.f;
	bool bFoundParam = GetParamValue(Data, ParameterName, ParamFloat);
	if(!bFoundParam)
	{
		ParamFloat = Constant;
	}

	if(ParamMode == DPM_Direct)
	{
		return ParamFloat;
	}
	else if(ParamMode == DPM_Abs)
	{
		ParamFloat = FMath::Abs(ParamFloat);
	}

	float Gradient;
	if(MaxInput <= MinInput)
		Gradient = 0.f;
	else
		Gradient = (MaxOutput - MinOutput)/(MaxInput - MinInput);

	float ClampedParam = FMath::Clamp(ParamFloat, MinInput, MaxInput);
	float Output = MinOutput + ((ClampedParam - MinInput) * Gradient);

	return Output;
}


UDistributionVectorParameterBase::UDistributionVectorParameterBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MaxInput = FVector(1.0f,1.0f,1.0f);
	MaxOutput = FVector(1.0f,1.0f,1.0f);

}

FVector UDistributionVectorParameterBase::GetValue( float F, UObject* Data, int32 Extreme, struct FRandomStream* InRandomStream ) const
{
	FVector ParamVector(0.f);
	bool bFoundParam = GetParamValue(Data, ParameterName, ParamVector);
	if(!bFoundParam)
	{
		ParamVector = Constant;
	}

	if(ParamModes[0] == DPM_Abs)
	{
		ParamVector.X = FMath::Abs(ParamVector.X);
	}

	if(ParamModes[1] == DPM_Abs)
	{
		ParamVector.Y = FMath::Abs(ParamVector.Y);
	}

	if(ParamModes[2] == DPM_Abs)
	{
		ParamVector.Z = FMath::Abs(ParamVector.Z);
	}

	FVector Gradient;
	if(MaxInput.X <= MinInput.X)
		Gradient.X = 0.f;
	else
		Gradient.X = (MaxOutput.X - MinOutput.X)/(MaxInput.X - MinInput.X);

	if(MaxInput.Y <= MinInput.Y)
		Gradient.Y = 0.f;
	else
		Gradient.Y = (MaxOutput.Y - MinOutput.Y)/(MaxInput.Y - MinInput.Y);

	if(MaxInput.Z <= MinInput.Z)
		Gradient.Z = 0.f;
	else
		Gradient.Z = (MaxOutput.Z - MinOutput.Z)/(MaxInput.Z - MinInput.Z);

	FVector ClampedParam;
	ClampedParam.X = FMath::Clamp(ParamVector.X, MinInput.X, MaxInput.X);
	ClampedParam.Y = FMath::Clamp(ParamVector.Y, MinInput.Y, MaxInput.Y);
	ClampedParam.Z = FMath::Clamp(ParamVector.Z, MinInput.Z, MaxInput.Z);

	FVector Output = MinOutput + ((ClampedParam - MinInput) * Gradient);

	if(ParamModes[0] == DPM_Direct)
	{
		Output.X = ParamVector.X;
	}

	if(ParamModes[1] == DPM_Direct)
	{
		Output.Y = ParamVector.Y;
	}

	if(ParamModes[2] == DPM_Direct)
	{
		Output.Z = ParamVector.Z;
	}

	return Output;
}

