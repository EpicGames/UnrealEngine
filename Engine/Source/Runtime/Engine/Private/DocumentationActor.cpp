// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DocumentationActor.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DocumentationActor)

#if WITH_EDITOR
#include "Components/MaterialBillboardComponent.h"
#include "IDocumentation.h"
#endif

ADocumentationActor::ADocumentationActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;	

#if WITH_EDITORONLY_DATA
	// Create a Material billboard to represent our actor
	Billboard = CreateDefaultSubobject<UMaterialBillboardComponent>(TEXT("BillboardComponent"));
	if (!IsRunningCommandlet() && (Billboard != NULL))
	{
		static ConstructorHelpers::FObjectFinder<UMaterial> MaterialAsset(TEXT("/Engine/EditorMaterials/HelpActorMaterial"));
		Billboard->AddElement(MaterialAsset.Object, nullptr, false, 32.0f, 32.0f, nullptr);
		Billboard->SetupAttachment(RootComponent);
	}
#endif //WITH_EDITORONLY_DATA
}

bool ADocumentationActor::OpenDocumentLink() const
{
	bool bOpened = false;

#if WITH_EDITOR
	bOpened = IDocumentation::Get()->Open(DocumentLink, FDocumentationSourceInfo(TEXT("doc_actors")));
#endif //WITH_EDITOR

	return bOpened;
}

bool ADocumentationActor::HasValidDocumentLink() const
{
	bool bDocumentValid = false;

#if WITH_EDITORONLY_DATA
	bDocumentValid = DocumentLink.IsEmpty() == false; 
#endif // WITH_EDITORONLY_DATA

	return bDocumentValid;
}

EDocumentationActorType::Type ADocumentationActor::GetLinkType() const
{
	return LinkType;
}

#if WITH_EDITOR
void ADocumentationActor::PostLoad()
{
	Super::PostLoad();

	UpdateLinkType();
}


void ADocumentationActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	UpdateLinkType();
}
#endif

void ADocumentationActor::UpdateLinkType()
{
#if WITH_EDITORONLY_DATA
	DocumentLink = DocumentLink.Replace(TEXT("\\"), TEXT("/"));	
	if (DocumentLink.IsEmpty() == true)
	{
		LinkType = EDocumentationActorType::None;
	}
	else if ((DocumentLink.StartsWith(TEXT("HTTP://")) == true) || (DocumentLink.StartsWith(TEXT("HTTPS://")) == true))
	{
		LinkType = EDocumentationActorType::URLLink;
	}
	else
	{
		LinkType = EDocumentationActorType::UDNLink;
	}
#else
	LinkType = EDocumentationActorType::None;
#endif
	
}

