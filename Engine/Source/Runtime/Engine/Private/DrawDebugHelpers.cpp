// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#include "DrawDebugHelpers.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"
#include "Components/LineBatchComponent.h"
#include "Engine/Canvas.h"
#include "GameFramework/HUD.h"

#if ENABLE_DRAW_DEBUG

ENGINE_API float GServerDrawDebugColorTintStrength = 0.75f;
ENGINE_API FLinearColor GServerDrawDebugColorTint(0.0f, 0.0f, 0.0f, 1.0f);

namespace UE::Private
{
	static void OnEnableDrawDebugHelpersChanged(IConsoleVariable* EnableDrawDebugHelpersVar)
	{
		// Clear out any persistently-drawn items if option is disabled
		if (!EnableDrawDebugHelpersVar->GetBool())
		{
			for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
			{
				FlushPersistentDebugLines(WorldContext.World());
				FlushDebugStrings(WorldContext.World());
			}
		}
	}

	static bool bEnableDrawDebugHelpers = true;
	static FAutoConsoleVariableRef CVarEnableDrawDebugHelpers(
		TEXT("r.EnableDrawDebugHelpers"),
		bEnableDrawDebugHelpers,
		TEXT(	"If true, then any 'DrawDebug' helper functions will be enabled.\nIf false, then the 'DrawDebug' helper functions will exit early and any persistent items will be removed.\nThis is only considered when ENABLE_DRAW_DEBUG is true"),
		FConsoleVariableDelegate::CreateStatic(&OnEnableDrawDebugHelpersChanged));
}

// Checks the value of the r.EnableDrawDebugHelpers cvar, and returns early if it is false.
#define RETURN_IF_DEBUG_DRAW_DISABLED		if (!UE::Private::bEnableDrawDebugHelpers) { return; }

#if WITH_EDITOR

FColor AdjustColorForServer(const FColor InColor)
{
	if (GServerDrawDebugColorTintStrength > 0.0f)
	{
		return FMath::Lerp(FLinearColor::FromSRGBColor(InColor), GServerDrawDebugColorTint, GServerDrawDebugColorTintStrength).ToFColor(/*bSRGB=*/ true);
	}
	else
	{
		return InColor;
	}
}

bool CanDrawServerDebugInContext(const FWorldContext& WorldContext)
{
	return
		(WorldContext.WorldType == EWorldType::PIE) &&
		(WorldContext.World() != nullptr) &&
		(WorldContext.World()->GetNetMode() == NM_Client) &&
		(WorldContext.GameViewport != nullptr) &&
		(WorldContext.GameViewport->EngineShowFlags.ServerDrawDebug);
}

#define UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(FunctionName, ...) \
		if (GIsEditor) \
		{ \
			for (const FWorldContext& WorldContext : GEngine->GetWorldContexts()) \
			{ \
				if (CanDrawServerDebugInContext(WorldContext)) \
				{ \
					FunctionName(WorldContext.World(), __VA_ARGS__); \
				} \
			} \
		}

#else

#define UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(FunctionName, ...)

#endif


void FlushPersistentDebugLines( const UWorld* InWorld )
{
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (InWorld)
		{
			if (ULineBatchComponent* WorldPersistentLineBatcher = InWorld->GetLineBatcher(UWorld::ELineBatcherType::WorldPersistent))
			{
				WorldPersistentLineBatcher->Flush();
			}
			if (ULineBatchComponent* ForegroundPersistentLineBatcher = InWorld->GetLineBatcher(UWorld::ELineBatcherType::ForegroundPersistent))
			{
				ForegroundPersistentLineBatcher->Flush();
			}
		}
	}
#if WITH_EDITOR
	else
	{
		if (GIsEditor)
		{
			for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
			{ 
				if (CanDrawServerDebugInContext(WorldContext))
				{
					FlushPersistentDebugLines(WorldContext.World());
				}
			} 
		}
	}
#endif
}

ULineBatchComponent* GetDebugLineBatcher( const UWorld* InWorld, bool bPersistentLines, float LifeTime, bool bDepthIsForeground )
{
	if (InWorld)
	{
		if (bPersistentLines || LifeTime > 0.f)
		{
			return bDepthIsForeground ? InWorld->GetLineBatcher(UWorld::ELineBatcherType::ForegroundPersistent) : InWorld->GetLineBatcher(UWorld::ELineBatcherType::WorldPersistent);
		}
		else
		{
			return bDepthIsForeground ? InWorld->GetLineBatcher(UWorld::ELineBatcherType::Foreground) : InWorld->GetLineBatcher(UWorld::ELineBatcherType::World);
		}
	}

	return nullptr;
}

static float GetDebugLineLifeTime(ULineBatchComponent* LineBatcher, float LifeTime, bool bPersistent)
{
	return bPersistent ? -1.0f : ((LifeTime > 0.f) ? LifeTime : LineBatcher->DefaultLifeTime);
}

void DrawDebugLine(const UWorld* InWorld, FVector const& LineStart, FVector const& LineEnd, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			float const LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawLine(LineStart, LineEnd, Color, DepthPriority, Thickness, LineLifeTime);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugLine, LineStart, LineEnd, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawDebugPoint(const UWorld* InWorld, FVector const& Position, float Size, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float PointLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawPoint(Position, Color.ReinterpretAsLinear(), Size, DepthPriority, PointLifeTime);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugPoint, Position, Size, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority);
	}
}

void DrawDebugDirectionalArrow(const UWorld* InWorld, FVector const& LineStart, FVector const& LineEnd, float ArrowSize, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			float const LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			if (ArrowSize <= 0)
			{
				ArrowSize = 10.f;
			}
			
			LineBatcher->DrawDirectionalArrow(LineStart, LineEnd, ArrowSize, Color, LineLifeTime, DepthPriority, Thickness);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugDirectionalArrow, LineStart, LineEnd, ArrowSize, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawDebugBox(const UWorld* InWorld, FVector const& Center, FVector const& Box, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float BoxLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawBox(Center, Box, Color, BoxLifeTime, DepthPriority, Thickness);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugBox, Center, Box, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawDebugBox(const UWorld* InWorld, FVector const& Center, FVector const& Box, const FQuat& Rotation, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			float const BoxLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawBox(Center, Box, Rotation, Color, BoxLifeTime, DepthPriority, Thickness);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugBox, Center, Box, Rotation, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}


void DrawDebugMesh(const UWorld* InWorld, TArray<FVector> const& Verts, TArray<int32> const& Indices, FColor const& Color, bool bPersistent, float LifeTime, uint8 DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistent, LifeTime, false))
		{
			float const ActualLifetime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistent);
			LineBatcher->DrawMesh(Verts, Indices, Color, DepthPriority, ActualLifetime);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugMesh, Verts, Indices, AdjustColorForServer(Color), bPersistent, LifeTime, DepthPriority);
	}
}

void DrawDebugSolidBox(const UWorld* InWorld, FBox const& Box, FColor const& Color, const FTransform& Transform, bool bPersistent, float LifeTime, uint8 DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistent, LifeTime, false))
		{
			float const ActualLifetime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistent);
			LineBatcher->DrawSolidBox(Box, Transform, Color, DepthPriority, ActualLifetime);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugSolidBox, Box, AdjustColorForServer(Color), Transform, bPersistent, LifeTime, DepthPriority);
	}
}

void DrawDebugSolidBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, FColor const& Color, bool bPersistent, float LifeTime, uint8 DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// No Rotation, so just use identity transform and build the box in the right place!
	FBox Box = FBox::BuildAABB(Center, Extent);

	DrawDebugSolidBox(InWorld, Box, Color, FTransform::Identity, bPersistent, LifeTime, DepthPriority);
}

void DrawDebugSolidBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, FQuat const& Rotation, FColor const& Color, bool bPersistent, float LifeTime, uint8 DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	FTransform Transform(Rotation, Center, FVector(1.0f, 1.0f, 1.0f));	// Build transform from Rotation, Center with uniform scale of 1.0.
	FBox Box = FBox::BuildAABB(FVector::ZeroVector, Extent);	// The Transform handles the Center location, so this box needs to be centered on origin.

	DrawDebugSolidBox(InWorld, Box, Color, Transform, bPersistent, LifeTime, DepthPriority);
}

/** Loc is an anchor point in the world to guide which part of the infinite plane to draw. */
void DrawDebugSolidPlane(const UWorld* InWorld, FPlane const& P, FVector const& Loc, float Size, FColor const& Color, bool bPersistent, float LifeTime, uint8 DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	DrawDebugSolidPlane(InWorld, P, Loc, FVector2D(Size, Size), Color, bPersistent, LifeTime, DepthPriority);
}

ENGINE_API void DrawDebugSolidPlane(const UWorld* InWorld, FPlane const& P, FVector const& Loc, FVector2D const& Extents, FColor const& Color, bool bPersistent/*=false*/, float LifeTime/*=-1*/, uint8 DepthPriority /*= 0*/)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if(GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		FVector const ClosestPtOnPlane = Loc - P.PlaneDot(Loc) * P;

		FVector U, V;
		P.FindBestAxisVectors(U, V);
		U *= Extents.Y;
		V *= Extents.X;

		TArray<FVector> Verts;
		Verts.AddUninitialized(4);
		Verts[0] = ClosestPtOnPlane + U + V;
		Verts[1] = ClosestPtOnPlane - U + V;
		Verts[2] = ClosestPtOnPlane + U - V;
		Verts[3] = ClosestPtOnPlane - U - V;

		TArray<int32> Indices;
		Indices.AddUninitialized(6);
		Indices[0] = 0; Indices[1] = 2; Indices[2] = 1;
		Indices[3] = 1; Indices[4] = 2; Indices[5] = 3;

		// plane quad
		DrawDebugMesh(InWorld, Verts, Indices, Color, bPersistent, LifeTime, DepthPriority);

		// arrow indicating normal
		DrawDebugDirectionalArrow(InWorld, ClosestPtOnPlane, ClosestPtOnPlane + P * 16.f, 8.f, FColor::White, bPersistent, LifeTime, DepthPriority);
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugSolidPlane, P, Loc, Extents, AdjustColorForServer(Color), bPersistent, LifeTime, DepthPriority);
	}
}

void DrawDebugCoordinateSystem(const UWorld* InWorld, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		FRotationMatrix R(AxisRot);
		FVector const X = R.GetScaledAxis( EAxis::X );
		FVector const Y = R.GetScaledAxis( EAxis::Y );
		FVector const Z = R.GetScaledAxis( EAxis::Z );

		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawLine(AxisLoc, AxisLoc + X*Scale, FColor::Red, DepthPriority, Thickness, LineLifeTime );
			LineBatcher->DrawLine(AxisLoc, AxisLoc + Y*Scale, FColor::Green, DepthPriority, Thickness, LineLifeTime );
			LineBatcher->DrawLine(AxisLoc, AxisLoc + Z*Scale, FColor::Blue, DepthPriority, Thickness, LineLifeTime );
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCoordinateSystem, AxisLoc, AxisRot, Scale, bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

ENGINE_API void DrawDebugCrosshairs(const UWorld* InWorld, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		FRotationMatrix R(AxisRot);
		FVector const X = 0.5f * R.GetScaledAxis(EAxis::X);
		FVector const Y = 0.5f * R.GetScaledAxis(EAxis::Y);
		FVector const Z = 0.5f * R.GetScaledAxis(EAxis::Z);

		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawLine(AxisLoc - X*Scale, AxisLoc + X*Scale, Color, DepthPriority, 0.f, LineLifeTime);
			LineBatcher->DrawLine(AxisLoc - Y*Scale, AxisLoc + Y*Scale, Color, DepthPriority, 0.f, LineLifeTime);
			LineBatcher->DrawLine(AxisLoc - Z*Scale, AxisLoc + Z*Scale, Color, DepthPriority, 0.f, LineLifeTime);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCrosshairs, AxisLoc, AxisRot, Scale, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority);
	}
}


static void InternalDrawDebugCircle(const UWorld* InWorld, const FMatrix& TransformMatrix, float Radius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness = 0.f)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
	{
		const float LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);

		// Need at least 4 segments
		Segments = FMath::Max(Segments, 4);
		const float AngleStep = 2.f * UE_PI / float(Segments);

		const FVector Center = TransformMatrix.GetOrigin();
		const FVector AxisY = TransformMatrix.GetScaledAxis(EAxis::Y);
		const FVector AxisZ = TransformMatrix.GetScaledAxis(EAxis::Z);

		TArray<FBatchedLine> Lines;
		Lines.Empty(Segments);

		float Angle = 0.f;
		while (Segments--)
		{
			const FVector Vertex1 = Center + Radius * (AxisY * FMath::Cos(Angle) + AxisZ * FMath::Sin(Angle));
			Angle += AngleStep;
			const FVector Vertex2 = Center + Radius * (AxisY * FMath::Cos(Angle) + AxisZ * FMath::Sin(Angle));
			Lines.Add(FBatchedLine(Vertex1, Vertex2, Color, LineLifeTime, Thickness, DepthPriority));
		}
		LineBatcher->DrawLines(Lines);
	}
}

void DrawDebugCircle(const UWorld* InWorld, const FMatrix& TransformMatrix, double Radius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness, bool bDrawAxis)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);

			InternalDrawDebugCircle(InWorld, TransformMatrix, Radius, Segments, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);

			if (bDrawAxis)
			{
				const FVector Center = TransformMatrix.GetOrigin();
				const FVector AxisY = TransformMatrix.GetScaledAxis( EAxis::Y );
				const FVector AxisZ = TransformMatrix.GetScaledAxis( EAxis::Z );

				TArray<FBatchedLine> Lines;
				Lines.Empty(2);
				Lines.Add(FBatchedLine(Center - Radius * AxisY, Center + Radius * AxisY, Color, LineLifeTime, Thickness, DepthPriority));
				Lines.Add(FBatchedLine(Center - Radius * AxisZ, Center + Radius * AxisZ, Color, LineLifeTime, Thickness, DepthPriority));
				LineBatcher->DrawLines(Lines);
			}
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCircle, TransformMatrix, Radius, Segments, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness, bDrawAxis);
	}
}

void DrawDebugCircle(const UWorld* InWorld, FVector Center, double Radius, int32 Segments, const FColor& Color, bool PersistentLines, float LifeTime, uint8 DepthPriority, float Thickness, FVector YAxis, FVector ZAxis, bool bDrawAxis)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	FMatrix TM;
	TM.SetOrigin(Center);
	TM.SetAxis(0, FVector(1,0,0));
	TM.SetAxis(1, YAxis);
	TM.SetAxis(2, ZAxis);
	
	DrawDebugCircle(
		InWorld,
		TM,
		Radius,
		Segments,
		Color,
		PersistentLines,
		LifeTime,
		DepthPriority,
		Thickness,
		bDrawAxis
	);
}

void DrawDebugCircleArc(const UWorld* InWorld, const FVector& Center, float Radius, const FVector& Direction, float AngleWidth, int32 Segments, const FColor& Color, bool PersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, PersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, PersistentLines);

			// Need at least 4 segments
			Segments = FMath::Max(Segments, 4);
			const float AngleStep = AngleWidth / float(Segments) * 2.f;

			FVector AxisY, AxisZ;
			FVector DirectionNorm = Direction.GetSafeNormal();
			DirectionNorm.FindBestAxisVectors(AxisZ, AxisY);

			TArray<FBatchedLine> Lines;
			Lines.Empty(Segments);
			float Angle = -AngleWidth;
			FVector PrevVertex = Center + Radius * (AxisY * -FMath::Sin(Angle) + DirectionNorm * FMath::Cos(Angle));
			while (Segments--)
			{
				Angle += AngleStep;
				FVector NextVertex = Center + Radius * (AxisY * -FMath::Sin(Angle) + DirectionNorm * FMath::Cos(Angle));
				Lines.Emplace(FBatchedLine(PrevVertex, NextVertex, Color, LineLifeTime, Thickness, DepthPriority));
				PrevVertex = NextVertex;
			}

			LineBatcher->DrawLines(Lines);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCircleArc, Center, Radius, Direction, AngleWidth, Segments, AdjustColorForServer(Color), PersistentLines, LifeTime, DepthPriority, Thickness);
	}

}

void DrawDebugCircleArc(const UWorld* InWorld, const FVector& Center, float Radius, const FQuat& Rotation, float AngleWidth, int32 Segments, const FColor& Color, bool PersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, PersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, PersistentLines);

			// Need at least 4 segments
			Segments = FMath::Max(Segments, 4);
			const float AngleStep = AngleWidth / static_cast<float>(Segments) * 2.f;

			const FVector AxisX = Rotation.Vector();
			const FVector AxisY = Rotation.GetAxisY();

			TArray<FBatchedLine> Lines;
			Lines.Reserve(Segments);
			float Angle = -AngleWidth;
			
			FVector PrevVertex = Center + Radius * (AxisX * FMath::Cos(Angle) + AxisY * FMath::Sin(Angle));
			
			while (Segments--)
			{
				Angle += AngleStep;
				const FVector NextVertex = Center + Radius * (AxisX * FMath::Cos(Angle) + AxisY * FMath::Sin(Angle));
				Lines.Emplace(FBatchedLine(PrevVertex, NextVertex, Color, LineLifeTime, Thickness, DepthPriority));
				PrevVertex = NextVertex;
			}

			LineBatcher->DrawLines(Lines);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCircleArc, Center, Radius, Rotation, AngleWidth, Segments, AdjustColorForServer(Color), PersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawDebug2DDonut(const UWorld* InWorld, const FMatrix& TransformMatrix, float InnerRadius, float OuterRadius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);

			// Need at least 4 segments
			Segments = FMath::Max((Segments - 4) / 2, 4);
			InternalDrawDebugCircle(InWorld, TransformMatrix, InnerRadius, Segments, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			InternalDrawDebugCircle(InWorld, TransformMatrix, OuterRadius, Segments, Color, bPersistentLines, LifeTime, DepthPriority, Thickness );
		
			const FVector Center = TransformMatrix.GetOrigin();
			const FVector AxisY = TransformMatrix.GetScaledAxis( EAxis::Y );
			const FVector AxisZ = TransformMatrix.GetScaledAxis( EAxis::Z );

			TArray<FBatchedLine> Lines;
			Lines.Empty(4);
			Lines.Add(FBatchedLine(Center - OuterRadius * AxisY, Center - InnerRadius * AxisY, Color, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Center + OuterRadius * AxisY, Center + InnerRadius * AxisY, Color, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Center - OuterRadius * AxisZ, Center - InnerRadius * AxisZ, Color, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Center + OuterRadius * AxisZ, Center + InnerRadius * AxisZ, Color, LineLifeTime, Thickness, DepthPriority));
			LineBatcher->DrawLines(Lines);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebug2DDonut, TransformMatrix, InnerRadius, OuterRadius, Segments, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawDebugSphere(const UWorld* InWorld, FVector const& Center, float Radius, int32 Segments, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float SphereLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawSphere(Center, Radius, Segments, Color, SphereLifeTime, DepthPriority, Thickness);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugSphere, Center, Radius, Segments, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawDebugCylinder(const UWorld* InWorld, FVector const& Start, FVector const& End, float Radius, int32 Segments, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			const float CylinderLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawCylinder(Start, End, Radius, Segments, Color, CylinderLifeTime, DepthPriority, Thickness);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCylinder, Start, End, Radius, Segments, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

/** Used by gameplay when defining a cone by a vertical and horizontal dot products. */
void DrawDebugAltCone(const UWorld* InWorld, FVector const& Origin, FRotator const& Rotation, float Length, float AngleWidth, float AngleHeight, FColor const& DrawColor, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			FRotationMatrix const RM(Rotation);
			FVector const AxisX = RM.GetScaledAxis(EAxis::X);
			FVector const AxisY = RM.GetScaledAxis(EAxis::Y);
			FVector const AxisZ = RM.GetScaledAxis(EAxis::Z);

			float const LineLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);

			FVector const EndPoint = Origin + AxisX * Length;
			FVector const Up = FMath::Tan(AngleHeight * 0.5f) * AxisZ * Length;
			FVector const Right = FMath::Tan(AngleWidth * 0.5f) * AxisY * Length;
			FVector const HalfUp = Up * 0.5f;
			FVector const HalfRight = Right * 0.5f;

			TArray<FBatchedLine> Lines;
			Lines.Empty();

			FVector A = EndPoint + Up - Right;
			FVector B = EndPoint + Up + Right;
			FVector C = EndPoint - Up + Right;
			FVector D = EndPoint - Up - Right;

			// Corners
			Lines.Add(FBatchedLine(Origin, A, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, B, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, C, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, D, DrawColor, LineLifeTime, Thickness, DepthPriority));

			// Further most plane/frame
			Lines.Add(FBatchedLine(A, B, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(B, C, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(C, D, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(D, A, DrawColor, LineLifeTime, Thickness, DepthPriority));

			// Mid points
			Lines.Add(FBatchedLine(Origin, EndPoint + Up, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, EndPoint - Up, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, EndPoint + Right, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, EndPoint - Right, DrawColor, LineLifeTime, Thickness, DepthPriority));

			// Inbetween
			Lines.Add(FBatchedLine(Origin, EndPoint + Up - HalfRight, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, EndPoint + Up + HalfRight, DrawColor, LineLifeTime, Thickness, DepthPriority));

			Lines.Add(FBatchedLine(Origin, EndPoint - Up - HalfRight, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, EndPoint - Up + HalfRight, DrawColor, LineLifeTime, Thickness, DepthPriority));

			Lines.Add(FBatchedLine(Origin, EndPoint + Right - HalfUp, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, EndPoint + Right + HalfUp, DrawColor, LineLifeTime, Thickness, DepthPriority));

			Lines.Add(FBatchedLine(Origin, EndPoint - Right - HalfUp, DrawColor, LineLifeTime, Thickness, DepthPriority));
			Lines.Add(FBatchedLine(Origin, EndPoint - Right + HalfUp, DrawColor, LineLifeTime, Thickness, DepthPriority));

			LineBatcher->DrawLines(Lines);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugAltCone, Origin, Rotation, Length, AngleWidth, AngleHeight, AdjustColorForServer(DrawColor), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawDebugCone(const UWorld* InWorld, FVector const& Origin, FVector const& Direction, float Length, float AngleWidth, float AngleHeight, int32 NumSides, FColor const& DrawColor, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		// this means foreground lines can't be persistent 
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			float const ConeLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawCone(Origin, Direction, Length, AngleWidth, AngleHeight, NumSides, DrawColor, ConeLifeTime, DepthPriority, Thickness);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCone, Origin, Direction, Length, AngleWidth, AngleHeight, NumSides, AdjustColorForServer(DrawColor), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawDebugString(const UWorld* InWorld, FVector const& TextLocation, const FString& Text, class AActor* TestBaseActor, FColor const& TextColor, float Duration, bool bDrawShadow, float FontScale)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		check((TestBaseActor == nullptr) || (TestBaseActor->GetWorld() == InWorld));
		AActor* BaseAct = (TestBaseActor != nullptr) ? TestBaseActor : InWorld->GetWorldSettings();

		// iterate through the player controller list
		for( FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator )
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController && PlayerController->MyHUD && PlayerController->Player)
			{
				PlayerController->MyHUD->AddDebugText(Text, BaseAct, Duration, TextLocation, TextLocation, TextColor, /*bSkipOverwriteCheck=*/ true, /*bAbsoluteLocation=*/ (TestBaseActor==nullptr), /*bKeepAttachedToActor=*/ false, nullptr, FontScale, bDrawShadow);
			}
		}
	}
	else
	{
		// We do a bit of converting here if the original call was relative, as there's a check() that the base actor is
		// in the same world as being rendered to (and it might be in a different position on client vs server anyways)
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugString, (TestBaseActor != nullptr) ? (TextLocation + TestBaseActor->GetActorLocation()) : TextLocation, Text, /*TestBaseActor=*/ nullptr, AdjustColorForServer(TextColor), Duration, bDrawShadow, FontScale);
	}
}

void FlushDebugStrings( const UWorld* InWorld )
{
	// iterate through the controller list
	for( FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		// if it's a player
		APlayerController* PlayerController = Iterator->Get();
		if (PlayerController && PlayerController->MyHUD)
		{
			PlayerController->MyHUD->RemoveAllDebugStrings();
		}
	}	
}

void DrawDebugFrustum(const UWorld* InWorld, const FMatrix& FrustumToWorld, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		FVector Vertices[2][2][2];
		for(uint32 Z = 0;Z < 2;Z++)
		{
			for(uint32 Y = 0;Y < 2;Y++)
			{
				for(uint32 X = 0;X < 2;X++)
				{
					FVector4 UnprojectedVertex = FrustumToWorld.TransformFVector4(
						FVector4(
						(X ? -1.0f : 1.0f),
						(Y ? -1.0f : 1.0f),
						(Z ?  0.0f : 1.0f),
						1.0f
						)
						);
					Vertices[X][Y][Z] = FVector(UnprojectedVertex) / UnprojectedVertex.W;
				}
			}
		}

		DrawDebugLine(InWorld, Vertices[0][0][0], Vertices[0][0][1],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[1][0][0], Vertices[1][0][1],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[0][1][0], Vertices[0][1][1],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[1][1][0], Vertices[1][1][1],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);

		DrawDebugLine(InWorld, Vertices[0][0][0], Vertices[0][1][0],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[1][0][0], Vertices[1][1][0],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[0][0][1], Vertices[0][1][1],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[1][0][1], Vertices[1][1][1],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);

		DrawDebugLine(InWorld, Vertices[0][0][0], Vertices[1][0][0],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[0][1][0], Vertices[1][1][0],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[0][0][1], Vertices[1][0][1],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
		DrawDebugLine(InWorld, Vertices[0][1][1], Vertices[1][1][1],Color,  bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugFrustum, FrustumToWorld, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}

void DrawCircle(const UWorld* InWorld, const FVector& Base, const FVector& X, const FVector& Y, const FColor& Color, double Radius, int32 NumSides, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	FVector	LastVertex = Base + X * Radius;

	for(int32 SideIndex = 0;SideIndex < NumSides;SideIndex++)
	{
		const FVector Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		DrawDebugLine(InWorld, LastVertex, Vertex, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
		LastVertex = Vertex;
	}
}

void DrawDebugCapsule(const UWorld* InWorld, FVector const& Center, float HalfHeight, float Radius, const FQuat& Rotation, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		if (ULineBatchComponent* const LineBatcher = GetDebugLineBatcher(InWorld, bPersistentLines, LifeTime, (DepthPriority == SDPG_Foreground)))
		{
			float const CapsuleLifeTime = GetDebugLineLifeTime(LineBatcher, LifeTime, bPersistentLines);
			LineBatcher->DrawCapsule(Center, HalfHeight, Radius, Rotation, Color, CapsuleLifeTime, DepthPriority, Thickness);
		}
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCapsule, Center, HalfHeight, Radius, Rotation, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority, Thickness);
	}
}


void DrawDebugCamera(const UWorld* InWorld, FVector const& Location, FRotator const& Rotation, float FOVDeg, float Scale, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	static float BaseScale = 4.f;
	static FVector BaseProportions(2.f, 1.f, 1.5f);

	// no debug line drawing on dedicated server
	if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
	{
		DrawDebugCoordinateSystem(InWorld, Location, Rotation, BaseScale*Scale, bPersistentLines, LifeTime, DepthPriority);
		FVector Extents = BaseProportions * BaseScale * Scale;
		DrawDebugBox(InWorld, Location, Extents, Rotation.Quaternion(), Color, bPersistentLines, LifeTime, DepthPriority);		// lifetime

		// draw "lens" portion
		FRotationTranslationMatrix Axes(Rotation, Location);
		FVector XAxis = Axes.GetScaledAxis( EAxis::X );
		FVector YAxis = Axes.GetScaledAxis( EAxis::Y );
		FVector ZAxis = Axes.GetScaledAxis( EAxis::Z ); 

		FVector LensPoint = Location + XAxis * Extents.X;
		float LensSize = BaseProportions.Z * Scale * BaseScale;
		float HalfLensSize = LensSize * FMath::Tan(FMath::DegreesToRadians(FOVDeg*0.5f));
		FVector Corners[4] = 
		{
			LensPoint + XAxis * LensSize + (YAxis * HalfLensSize) + (ZAxis * HalfLensSize),
			LensPoint + XAxis * LensSize + (YAxis * HalfLensSize) - (ZAxis * HalfLensSize),
			LensPoint + XAxis * LensSize - (YAxis * HalfLensSize) - (ZAxis * HalfLensSize),
			LensPoint + XAxis * LensSize - (YAxis * HalfLensSize) + (ZAxis * HalfLensSize),
		};

		DrawDebugLine(InWorld, LensPoint, Corners[0], Color, bPersistentLines, LifeTime, DepthPriority);
		DrawDebugLine(InWorld, LensPoint, Corners[1], Color, bPersistentLines, LifeTime, DepthPriority);
		DrawDebugLine(InWorld, LensPoint, Corners[2], Color, bPersistentLines, LifeTime, DepthPriority);
		DrawDebugLine(InWorld, LensPoint, Corners[3], Color, bPersistentLines, LifeTime, DepthPriority);

		DrawDebugLine(InWorld, Corners[0], Corners[1], Color, bPersistentLines, LifeTime, DepthPriority);
		DrawDebugLine(InWorld, Corners[1], Corners[2], Color, bPersistentLines, LifeTime, DepthPriority);
		DrawDebugLine(InWorld, Corners[2], Corners[3], Color, bPersistentLines, LifeTime, DepthPriority);
		DrawDebugLine(InWorld, Corners[3], Corners[0], Color, bPersistentLines, LifeTime, DepthPriority);
	}
	else
	{
		UE_DRAW_SERVER_DEBUG_ON_EACH_CLIENT(DrawDebugCamera, Location, Rotation, FOVDeg, Scale, AdjustColorForServer(Color), bPersistentLines, LifeTime, DepthPriority);
	}
}

// https://en.wikipedia.org/wiki/Centripetal_Catmull%E2%80%93Rom_spline
void DrawCentripetalCatmullRomSpline(const UWorld* InWorld, TConstArrayView<FVector> Points, FColor const& Color, float Alpha, int32 NumSamplesPerSegment, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	TConstArrayView<FColor> Colors(&Color, 1);
	DrawCentripetalCatmullRomSpline(InWorld, Points, Colors, Alpha, NumSamplesPerSegment, bPersistentLines, LifeTime, DepthPriority, Thickness);
}

void DrawCentripetalCatmullRomSpline(const UWorld* InWorld, TConstArrayView<FVector> Points, TConstArrayView<FColor> Colors, float Alpha, int32 NumSamplesPerSegment, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	const int32 NumPoints = Points.Num();
	const int32 NumColors = Colors.Num();
	if (NumPoints > 1 && NumColors > 0)
	{
		auto GetT = [](float T, float Alpha, const FVector& P0, const FVector& P1)
		{
			const FVector P1P0 = P1 - P0;
			const float Dot = P1P0 | P1P0;
			const float Pow = FMath::Pow(Dot, Alpha * .5f);
			return Pow + T;
		};

		auto LerpColor = [](FColor A, FColor B, float T) -> FColor
		{
			return FColor(
				FMath::RoundToInt(float(A.R) * (1.f - T) + float(B.R) * T),
				FMath::RoundToInt(float(A.G) * (1.f - T) + float(B.G) * T),
				FMath::RoundToInt(float(A.B) * (1.f - T) + float(B.B) * T),
				FMath::RoundToInt(float(A.A) * (1.f - T) + float(B.A) * T));
		};

		FVector PrevPoint = Points[0];
		for (int i = 0; i < NumPoints - 1; ++i)
		{
			const FVector& P0 = Points[FMath::Max(i - 1, 0)];
			const FVector& P1 = Points[i];
			const FVector& P2 = Points[i + 1];
			const FVector& P3 = Points[FMath::Min(i + 2, NumPoints - 1)];

			const float T0 = 0.0f;
			const float T1 = GetT(T0, Alpha, P0, P1);
			const float T2 = GetT(T1, Alpha, P1, P2);
			const float T3 = GetT(T2, Alpha, P2, P3);

			const float T1T0 = T1 - T0;
			const float T2T1 = T2 - T1;
			const float T3T2 = T3 - T2;
			const float T2T0 = T2 - T0;
			const float T3T1 = T3 - T1;

			const bool bIsNearlyZeroT1T0 = FMath::IsNearlyZero(T1T0, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT2T1 = FMath::IsNearlyZero(T2T1, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT3T2 = FMath::IsNearlyZero(T3T2, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT2T0 = FMath::IsNearlyZero(T2T0, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT3T1 = FMath::IsNearlyZero(T3T1, UE_KINDA_SMALL_NUMBER);

			const FColor Color1 = Colors[FMath::Min(i, NumColors - 1)];
			const FColor Color2 = Colors[FMath::Min(i + 1, NumColors - 1)];

			for (int SampleIndex = 1; SampleIndex < NumSamplesPerSegment; ++SampleIndex)
			{
				const float ParametricDistance = float(SampleIndex) / float(NumSamplesPerSegment - 1);

				const float T = FMath::Lerp(T1, T2, ParametricDistance);

				const FVector A1 = bIsNearlyZeroT1T0 ? P0 : (T1 - T) / T1T0 * P0 + (T - T0) / T1T0 * P1;
				const FVector A2 = bIsNearlyZeroT2T1 ? P1 : (T2 - T) / T2T1 * P1 + (T - T1) / T2T1 * P2;
				const FVector A3 = bIsNearlyZeroT3T2 ? P2 : (T3 - T) / T3T2 * P2 + (T - T2) / T3T2 * P3;
				const FVector B1 = bIsNearlyZeroT2T0 ? A1 : (T2 - T) / T2T0 * A1 + (T - T0) / T2T0 * A2;
				const FVector B2 = bIsNearlyZeroT3T1 ? A2 : (T3 - T) / T3T1 * A2 + (T - T1) / T3T1 * A3;
				const FVector Point = bIsNearlyZeroT2T1 ? B1 : (T2 - T) / T2T1 * B1 + (T - T1) / T2T1 * B2;

				DrawDebugLine(InWorld, PrevPoint, Point, LerpColor(Color1, Color2, ParametricDistance), bPersistentLines, LifeTime, DepthPriority, Thickness);

				PrevPoint = Point;
			}
		}
	}
}

void DrawDebugFloatHistory(UWorld const & WorldRef, FDebugFloatHistory const & FloatHistory, FTransform const & DrawTransform, FVector2D const & DrawSize, FColor const & DrawColor, bool const & bPersistent, float const & LifeTime, uint8 const & DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	int const NumSamples = FloatHistory.GetNumSamples();
	if (NumSamples >= 2)
	{
		FVector DrawLocation = DrawTransform.GetLocation();
		FVector const AxisX = DrawTransform.GetUnitAxis(EAxis::Y);
		FVector const AxisY = DrawTransform.GetUnitAxis(EAxis::Z);
		FVector const AxisXStep = AxisX *  DrawSize.X / float(NumSamples);
		FVector const AxisYStep = AxisY *  DrawSize.Y / FMath::Max(FloatHistory.GetMinMaxRange(), UE_KINDA_SMALL_NUMBER);

		// Frame
		DrawDebugLine(&WorldRef, DrawLocation, DrawLocation + AxisX * DrawSize.X, DrawColor, bPersistent, LifeTime, DepthPriority);
		DrawDebugLine(&WorldRef, DrawLocation, DrawLocation + AxisY * DrawSize.Y, DrawColor, bPersistent, LifeTime, DepthPriority);
		DrawDebugLine(&WorldRef, DrawLocation + AxisY * DrawSize.Y, DrawLocation + AxisX * DrawSize.X + AxisY * DrawSize.Y, DrawColor, bPersistent, LifeTime, DepthPriority);
		DrawDebugLine(&WorldRef, DrawLocation + AxisX * DrawSize.X, DrawLocation + AxisX * DrawSize.X + AxisY * DrawSize.Y, DrawColor, bPersistent, LifeTime, DepthPriority);

		TArray<float> const & Samples = FloatHistory.GetSamples();

		TArray<FVector> Verts;
		Verts.AddUninitialized(NumSamples * 2);

		TArray<int32> Indices;
		Indices.AddUninitialized((NumSamples - 1) * 6);

		Verts[0] = DrawLocation;
		Verts[1] = DrawLocation + AxisYStep * Samples[0];

		for (int HistoryIndex = 1; HistoryIndex < NumSamples; HistoryIndex++)
		{
			DrawLocation += AxisXStep;

			int const VertIndex = (HistoryIndex - 1) * 2;
			Verts[VertIndex + 2] = DrawLocation;
			Verts[VertIndex + 3] = DrawLocation + AxisYStep * FMath::Clamp(Samples[HistoryIndex], FloatHistory.GetMinValue(), FloatHistory.GetMaxValue());

			int const StartIndex = (HistoryIndex - 1) * 6;
			Indices[StartIndex + 0] = VertIndex + 0; Indices[StartIndex + 1] = VertIndex + 1; Indices[StartIndex + 2] = VertIndex + 3;
			Indices[StartIndex + 3] = VertIndex + 0; Indices[StartIndex + 4] = VertIndex + 3; Indices[StartIndex + 5] = VertIndex + 2;
		}

		DrawDebugMesh(&WorldRef, Verts, Indices, DrawColor, bPersistent, LifeTime, DepthPriority);
	}
}

void DrawDebugFloatHistory(UWorld const & WorldRef, FDebugFloatHistory const & FloatHistory, FVector const & DrawLocation, FVector2D const & DrawSize, FColor const & DrawColor, bool const & bPersistent, float const & LifeTime, uint8 const & DepthPriority)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	APlayerController * PlayerController = WorldRef.GetGameInstance() != nullptr ? WorldRef.GetGameInstance()->GetFirstLocalPlayerController() : nullptr;
	FRotator const DrawRotation = (PlayerController && PlayerController->PlayerCameraManager) ? PlayerController->PlayerCameraManager->GetCameraRotation() : FRotator(0, 0, 0);

	FTransform const DrawTransform(DrawRotation, DrawLocation);
	DrawDebugFloatHistory(WorldRef, FloatHistory, DrawTransform, DrawSize, DrawColor, bPersistent, LifeTime, DepthPriority);
}

//////////////////////////////////////////////////////////////////
// Debug draw canvas operations

void DrawDebugCanvas2DLine(UCanvas* Canvas, const FVector& Start, const FVector& End, const FLinearColor& LineColor)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	FCanvasLineItem LineItem;
	LineItem.Origin = Start;
	LineItem.EndPos = End;
	LineItem.SetColor(LineColor);

	LineItem.Draw(Canvas->Canvas);
}

void DrawDebugCanvasLine(UCanvas* Canvas, const FVector& Start, const FVector& End, const FLinearColor& LineColor)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	DrawDebugCanvas2DLine(Canvas, Canvas->Project(Start), Canvas->Project(End), LineColor);
}

void DrawDebugCanvasCircle(UCanvas* Canvas, const FVector& Base, const FVector& X, const FVector& Y, FColor Color, float Radius, int32 NumSides)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	FVector	LastVertex = Base + X * Radius;

	for(int32 SideIndex = 0;SideIndex < NumSides;SideIndex++)
	{
		const FVector Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		DrawDebugCanvasLine(Canvas, LastVertex, Vertex, Color);
		LastVertex = Vertex;
	}
}

void DrawDebugCanvasHalfCircle(UCanvas* Canvas, const FVector& Base, const FVector& X, const FVector& Y, FColor Color, float Radius, int32 NumSides)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	FVector	LastVertex = Base + X * Radius;

	for (int32 SideIndex = 0;SideIndex < NumSides / 2;SideIndex++)
	{
		const FVector Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		DrawDebugCanvasLine(Canvas, LastVertex, Vertex, Color);
		LastVertex = Vertex;
	}
}

void DrawDebugCanvasWireSphere(UCanvas* Canvas, const FVector& Base, FColor Color, float Radius, int32 NumSides)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	DrawDebugCanvasCircle(Canvas, Base, FVector(1,0,0), FVector(0,1,0), Color, Radius, NumSides);
	DrawDebugCanvasCircle(Canvas, Base, FVector(1,0,0), FVector(0,0,1), Color, Radius, NumSides);
	DrawDebugCanvasCircle(Canvas, Base, FVector(0,1,0), FVector(0,0,1), Color, Radius, NumSides);
}

void DrawDebugCanvasWireCone(UCanvas* Canvas, const FTransform& Transform, float ConeRadius, float ConeAngle, int32 ConeSides, FColor Color)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	static const float TwoPI = 2.0f * UE_PI;
	static const float ToRads = UE_PI / 180.0f;
	static const float MaxAngle = 89.0f * ToRads + 0.001f;
	const float ClampedConeAngle = FMath::Clamp(ConeAngle * ToRads, 0.001f, MaxAngle);
	const float SinClampedConeAngle = FMath::Sin( ClampedConeAngle );
	const float CosClampedConeAngle = FMath::Cos( ClampedConeAngle );
	const FVector ConeDirection(1,0,0);
	const FVector ConeUpVector(0,1,0);
	const FVector ConeLeftVector(0,0,1);

	TArray<FVector> Verts;
	Verts.AddUninitialized( ConeSides );

	for ( int32 i = 0 ; i < Verts.Num() ; ++i )
	{
		const float Theta = static_cast<float>( (TwoPI * i) / Verts.Num() );
		Verts[i] = (ConeDirection * (ConeRadius * CosClampedConeAngle)) +
			((SinClampedConeAngle * ConeRadius * FMath::Cos( Theta )) * ConeUpVector) +
			((SinClampedConeAngle * ConeRadius * FMath::Sin( Theta )) * ConeLeftVector);
	}

	// Transform to world space.
	for ( int32 i = 0 ; i < Verts.Num() ; ++i )
	{
		Verts[i] = Transform.TransformPosition( Verts[i] );
	}

	// Draw spokes.
	for ( int32 i = 0 ; i < Verts.Num(); ++i )
	{
		DrawDebugCanvasLine( Canvas, Transform.GetLocation(), Verts[i], Color );
	}

	// Draw rim.
	for ( int32 i = 0 ; i < Verts.Num()-1 ; ++i )
	{
		DrawDebugCanvasLine( Canvas, Verts[i], Verts[i+1], Color );
	}
	DrawDebugCanvasLine( Canvas, Verts[Verts.Num()-1], Verts[0], Color );
}

void DrawDebugCanvasWireBox(UCanvas* Canvas, const FMatrix& Transform, const FBox& Box, FColor Color)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	const FVector Vertices[] =
	{
		Transform.TransformPosition(FVector(Box.Min.X, Box.Min.Y, Box.Min.Z)),
		Transform.TransformPosition(FVector(Box.Min.X, Box.Min.Y, Box.Max.Z)),
		Transform.TransformPosition(FVector(Box.Min.X, Box.Max.Y, Box.Min.Z)),
		Transform.TransformPosition(FVector(Box.Min.X, Box.Max.Y, Box.Max.Z)),
		Transform.TransformPosition(FVector(Box.Max.X, Box.Min.Y, Box.Min.Z)),
		Transform.TransformPosition(FVector(Box.Max.X, Box.Min.Y, Box.Max.Z)),
		Transform.TransformPosition(FVector(Box.Max.X, Box.Max.Y, Box.Min.Z)),
		Transform.TransformPosition(FVector(Box.Max.X, Box.Max.Y, Box.Max.Z))
	};

	const FIntVector2 Edges[] =
	{
		{ 0, 1 }, { 2, 3 },	{ 4, 5 }, { 6, 7 },
		{ 0, 4 }, { 4, 6 },	{ 6, 2 }, { 2, 0 },
		{ 1, 5 }, { 5, 7 },	{ 7, 3 }, { 3, 1 }
	};

	for (const FIntVector2& Edge : Edges)
	{
		DrawDebugCanvasLine(Canvas, Vertices[Edge.X], Vertices[Edge.Y], Color);
	}
}

void DrawDebugCanvasCapsule(UCanvas* Canvas, const FMatrix& Transform, float HalfLength, float Radius, const FColor& LineColor)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	constexpr int32 DrawCollisionSides = 16;

	FVector Origin = Transform.GetOrigin();
	FVector XAxis = Transform.GetScaledAxis(EAxis::X);
	FVector YAxis = Transform.GetScaledAxis(EAxis::Y);
	FVector ZAxis = Transform.GetScaledAxis(EAxis::Z);

	// Draw top and bottom circles
	float HalfAxis = FMath::Max<float>(HalfLength - Radius, 1.f);
	FVector TopEnd = Origin + HalfAxis * ZAxis;
	FVector BottomEnd = Origin - HalfAxis * ZAxis;

	DrawDebugCanvasCircle(Canvas, TopEnd, XAxis, YAxis, LineColor, Radius, DrawCollisionSides);
	DrawDebugCanvasCircle(Canvas, BottomEnd, XAxis, YAxis, LineColor, Radius, DrawCollisionSides);

	// Draw domed caps
	DrawDebugCanvasHalfCircle(Canvas, TopEnd, YAxis, ZAxis, LineColor, Radius, DrawCollisionSides);
	DrawDebugCanvasHalfCircle(Canvas, TopEnd, XAxis, ZAxis, LineColor, Radius, DrawCollisionSides);

	FVector NegZAxis = -ZAxis;

	DrawDebugCanvasHalfCircle(Canvas, BottomEnd, YAxis, NegZAxis, LineColor, Radius, DrawCollisionSides);
	DrawDebugCanvasHalfCircle(Canvas, BottomEnd, XAxis, NegZAxis, LineColor, Radius, DrawCollisionSides);

	// Draw connected lines
	DrawDebugCanvasLine(Canvas, TopEnd + Radius * XAxis, BottomEnd + Radius * XAxis, LineColor);
	DrawDebugCanvasLine(Canvas, TopEnd - Radius * XAxis, BottomEnd - Radius * XAxis, LineColor);
	DrawDebugCanvasLine(Canvas, TopEnd + Radius * YAxis, BottomEnd + Radius * YAxis, LineColor);
	DrawDebugCanvasLine(Canvas, TopEnd - Radius * YAxis, BottomEnd - Radius * YAxis, LineColor);
}

//
// Canvas 2D
//

void DrawDebugCanvas2DLine(UCanvas* Canvas, const FVector2D& StartPosition, const FVector2D& EndPosition, const FLinearColor& LineColor, const float& LineThickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	if (Canvas)
	{
		FCanvasLineItem LineItem(StartPosition, EndPosition);
		LineItem.LineThickness = LineThickness;
		LineItem.SetColor(LineColor);
		Canvas->DrawItem(LineItem);
	}
}

void DrawDebugCanvas2DCircle(UCanvas* Canvas, const FVector2D& Center, float Radius, int32 NumSides, const FLinearColor& LineColor, const float& LineThickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	FVector2D AxisX(1.f, 0.f);
	FVector2D AxisY(0.f, -1.f);
	FVector2D LastVertex = Center + AxisX * Radius;

	for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
	{
		const FVector2D Vertex = Center + (AxisX * FMath::Cos(AngleDelta * (SideIndex + 1)) + AxisY * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		DrawDebugCanvas2DLine(Canvas, LastVertex, Vertex, LineColor, LineThickness);
		LastVertex = Vertex;
	}
}

void DrawDebugCanvas2DBox(UCanvas* Canvas, const FBox2D& Box, const FLinearColor& LineColor, const float& LineThickness)
{
	RETURN_IF_DEBUG_DRAW_DISABLED;
	
	if (Canvas)
	{
		FCanvasBoxItem BoxItem(Box.Min, Box.GetSize());
		BoxItem.LineThickness = LineThickness;
		BoxItem.SetColor(LineColor);

		Canvas->DrawItem(BoxItem);
	}
}

#endif // ENABLE_DRAW_DEBUG
