// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DynamicBlueprintBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicBlueprintBinding)


UDynamicBlueprintBinding::UDynamicBlueprintBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


