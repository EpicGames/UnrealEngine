// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DynamicMeshBuilder.cpp: Dynamic mesh builder implementation.
=============================================================================*/

#include "DynamicMeshBuilder.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "LocalVertexFactory.h"
#include "MeshBuilderOneFrameResources.h"
#include "Math/Vector2DHalf.h"
#include "ResourcePool.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "RHIResourceUtils.h"

class FGlobalDynamicMeshPoolPolicy
{
public:
	/** Buffers are created with a simple byte size */
	typedef uint32 CreationArguments;
	enum
	{
		NumSafeFrames = 3, /** Number of frames to leaves buffers before reclaiming/reusing */
		NumPoolBucketSizes = 16, /** Number of pool buckets */
		NumToDrainPerFrame = 100, /** Max. number of resources to cull in a single frame */
		CullAfterFramesNum = 10 /** Resources are culled if unused for more frames than this */
	};
	
	/** Get the pool bucket index from the size
	 * @param Size the number of bytes for the resource
	 * @returns The bucket index.
	 */
	uint32 GetPoolBucketIndex(uint32 Size)
	{
		unsigned long Lower = 0;
		unsigned long Upper = NumPoolBucketSizes;
		unsigned long Middle;
		
		do
		{
			Middle = ( Upper + Lower ) >> 1;
			if( Size <= BucketSizes[Middle-1] )
			{
				Upper = Middle;
			}
			else
			{
				Lower = Middle;
			}
		}
		while( Upper - Lower > 1 );
		
		check( Size <= BucketSizes[Lower] );
		check( (Lower == 0 ) || ( Size > BucketSizes[Lower-1] ) );
		
		return Lower;
	}
	
	/** Get the pool bucket size from the index
	 * @param Bucket the bucket index
	 * @returns The bucket size.
	 */
	uint32 GetPoolBucketSize(uint32 Bucket)
	{
		check(Bucket < NumPoolBucketSizes);
		return BucketSizes[Bucket];
	}
	
private:
	/** The bucket sizes */
	static uint32 BucketSizes[NumPoolBucketSizes];
};

uint32 FGlobalDynamicMeshPoolPolicy::BucketSizes[NumPoolBucketSizes] = {
	64, 128, 256, 512, 1024, 2048, 4096,
	8*1024, 16*1024, 32*1024, 64*1024, 128*1024, 256*1024,
	512*1024, 1*1024*1024, 2*1024*1024
};

#if PLATFORM_USES_GLES
typedef uint16 DynamicMeshIndexType;
#else
typedef int32 DynamicMeshIndexType;
#endif

class FGlobalDynamicMeshIndexPolicy : public FGlobalDynamicMeshPoolPolicy
{
public:
	enum
	{
		NumSafeFrames = FGlobalDynamicMeshPoolPolicy::NumSafeFrames,
		NumPoolBuckets = FGlobalDynamicMeshPoolPolicy::NumPoolBucketSizes,
		NumToDrainPerFrame = FGlobalDynamicMeshPoolPolicy::NumToDrainPerFrame,
		CullAfterFramesNum = FGlobalDynamicMeshPoolPolicy::CullAfterFramesNum
	};
	
	/** Creates the resource
	 * @param Args The buffer size in bytes.
	 * @returns A suitably sized buffer or NULL on failure.
	 */
	FBufferRHIRef CreateResource(FRHICommandListBase& RHICmdList, FGlobalDynamicMeshPoolPolicy::CreationArguments Args)
	{
		FGlobalDynamicMeshPoolPolicy::CreationArguments BufferSize = GetPoolBucketSize(GetPoolBucketIndex(Args));
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex(TEXT("FGlobalDynamicMeshIndexPolicy"), BufferSize, sizeof(DynamicMeshIndexType))
			// The use of BUF_Static is deliberate - on OS X the buffer backing-store orphaning & reallocation will dominate execution time
			// so to avoid this we don't reuse a buffer for several frames, thereby avoiding the pipeline stall and the reallocation cost.
			.AddUsage(EBufferUsageFlags::Static)
			.DetermineInitialState();

		return RHICmdList.CreateBuffer(CreateDesc);
	}
	
	/** Gets the arguments used to create resource
	 * @param Resource The buffer to get data for.
	 * @returns The arguments used to create the buffer.
	 */
	FGlobalDynamicMeshPoolPolicy::CreationArguments GetCreationArguments(FBufferRHIRef Resource)
	{
		return (Resource->GetSize());
	}
	
	/** Frees the resource
	 * @param Resource The buffer to prepare for release from the pool permanently.
	 */
	void FreeResource(FBufferRHIRef Resource)
	{
	}
};

class FGlobalDynamicMeshIndexPool : public TRenderResourcePool<FBufferRHIRef, FGlobalDynamicMeshIndexPolicy, FGlobalDynamicMeshPoolPolicy::CreationArguments>
{
public:
	/** Destructor */
	virtual ~FGlobalDynamicMeshIndexPool()
	{
	}
	
public: // From FTickableObjectRenderThread
	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FGlobalDynamicMeshIndexPool, STATGROUP_Tickables);
	}
};
TGlobalResource<FGlobalDynamicMeshIndexPool> GDynamicMeshIndexPool;

class FGlobalDynamicMeshVertexPolicy : public FGlobalDynamicMeshPoolPolicy
{
public:
	enum
	{
		NumSafeFrames = FGlobalDynamicMeshPoolPolicy::NumSafeFrames,
		NumPoolBuckets = FGlobalDynamicMeshPoolPolicy::NumPoolBucketSizes,
		NumToDrainPerFrame = FGlobalDynamicMeshPoolPolicy::NumToDrainPerFrame,
		CullAfterFramesNum = FGlobalDynamicMeshPoolPolicy::CullAfterFramesNum
	};
	
	/** Creates the resource
	 * @param Args The buffer size in bytes.
	 * @returns A suitably sized buffer or NULL on failure.
	 */
	FBufferRHIRef CreateResource(FRHICommandListBase& RHICmdList, FGlobalDynamicMeshPoolPolicy::CreationArguments Args)
	{
		FGlobalDynamicMeshPoolPolicy::CreationArguments BufferSize = GetPoolBucketSize(GetPoolBucketIndex(Args));

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("FGlobalDynamicMeshVertexPolicy"), BufferSize)
			.AddUsage(EBufferUsageFlags::Volatile | EBufferUsageFlags::ShaderResource)
			.DetermineInitialState();

		return RHICmdList.CreateBuffer(CreateDesc);
	}
	
	/** Gets the arguments used to create resource
	 * @param Resource The buffer to get data for.
	 * @returns The arguments used to create the buffer.
	 */
	FGlobalDynamicMeshPoolPolicy::CreationArguments GetCreationArguments(FBufferRHIRef Resource)
	{
		return (Resource->GetSize());
	}
	
	/** Frees the resource
	 * @param Resource The buffer to prepare for release from the pool permanently.
	 */
	void FreeResource(FBufferRHIRef Resource)
	{
	}
};

class FGlobalDynamicMeshVertexPool : public TRenderResourcePool<FBufferRHIRef, FGlobalDynamicMeshVertexPolicy, FGlobalDynamicMeshPoolPolicy::CreationArguments>
{
public:
	/** Destructor */
	virtual ~FGlobalDynamicMeshVertexPool()
	{
	}
	
public: // From FTickableObjectRenderThread
	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FGlobalDynamicMeshVertexPool, STATGROUP_Tickables);
	}
};
TGlobalResource<FGlobalDynamicMeshVertexPool> GDynamicMeshVertexPool;

FDynamicMeshIndexBuffer32::FDynamicMeshIndexBuffer32() = default;
FDynamicMeshIndexBuffer32::~FDynamicMeshIndexBuffer32() = default;

void FDynamicMeshIndexBuffer32::InitRHI(FRHICommandListBase& RHICmdList)
{
	IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FDynamicMeshIndexBuffer32"), UsageFlags, MakeConstArrayView(Indices));
}

FDynamicMeshIndexBuffer16::FDynamicMeshIndexBuffer16() = default;
FDynamicMeshIndexBuffer16::~FDynamicMeshIndexBuffer16() = default;

void FDynamicMeshIndexBuffer16::InitRHI(FRHICommandListBase& RHICmdList)
{
	IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FDynamicMeshIndexBuffer16"), UsageFlags, MakeConstArrayView(Indices));
}

/** FDynamicMeshBufferAllocator's base implementation. It always reallocates new buffers. */
FDynamicMeshBufferAllocator::~FDynamicMeshBufferAllocator()
{
}

int32 FDynamicMeshBufferAllocator::GetIndexBufferSize(uint32 NumElements) const
{
	return NumElements * sizeof(DynamicMeshIndexType);
}

int32 FDynamicMeshBufferAllocator::GetVertexBufferSize(uint32 Stride, uint32 NumElements) const
{
	return NumElements * Stride;
}

FBufferRHIRef FDynamicMeshBufferAllocator::AllocIndexBuffer(uint32 NumElements)
{
	return AllocIndexBuffer(FRHICommandListImmediate::Get(), NumElements);
}

FBufferRHIRef FDynamicMeshBufferAllocator::AllocIndexBuffer(FRHICommandListBase& RHICmdList, uint32 NumElements)
{
	uint32 SizeInBytes = GetIndexBufferSize(NumElements);

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateIndex(TEXT("FDynamicMeshBufferAllocator"), SizeInBytes, sizeof(DynamicMeshIndexType))
		.AddUsage(EBufferUsageFlags::Volatile)
		.DetermineInitialState();

	return RHICmdList.CreateBuffer(CreateDesc);
}

void FDynamicMeshBufferAllocator::ReleaseIndexBuffer(FBufferRHIRef& IndexBufferRHI)
{
	IndexBufferRHI = nullptr;
}

FBufferRHIRef FDynamicMeshBufferAllocator::AllocVertexBuffer(uint32 Stride, uint32 NumElements)
{
	return AllocVertexBuffer(FRHICommandListImmediate::Get(), Stride, NumElements);
}

FBufferRHIRef FDynamicMeshBufferAllocator::AllocVertexBuffer(FRHICommandListBase& RHICmdList, uint32 Stride, uint32 NumElements)
{
	uint32 SizeInBytes = GetVertexBufferSize(Stride, NumElements);

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FDynamicMeshBufferAllocator"), SizeInBytes)
		.AddUsage(EBufferUsageFlags::Volatile | EBufferUsageFlags::ShaderResource)
		.DetermineInitialState();

	return RHICmdList.CreateBuffer(CreateDesc);
}

void FDynamicMeshBufferAllocator::ReleaseVertexBuffer(FBufferRHIRef& VertexBufferRHI)
{
	VertexBufferRHI = nullptr;
}

/** This is our default implementation using GDynamicMeshIndexPool. */
class FPooledDynamicMeshBufferAllocator : public FDynamicMeshBufferAllocator
{
	virtual FBufferRHIRef AllocIndexBuffer(FRHICommandListBase& RHICmdList, uint32 NumElements) override
	{
		uint32 SizeInBytes = NumElements * sizeof(DynamicMeshIndexType);
		if (SizeInBytes <= FGlobalDynamicMeshIndexPolicy().GetPoolBucketSize(FGlobalDynamicMeshIndexPolicy::NumPoolBuckets - 1))
		{
			return GDynamicMeshIndexPool.CreatePooledResource(RHICmdList, SizeInBytes);
		}

		return FDynamicMeshBufferAllocator::AllocIndexBuffer(RHICmdList, NumElements);
	}

	virtual void ReleaseIndexBuffer(FBufferRHIRef& IndexBufferRHI)
	{
		if (IsValidRef(IndexBufferRHI))
		{
			if (IndexBufferRHI->GetSize() <= FGlobalDynamicMeshIndexPolicy().GetPoolBucketSize(FGlobalDynamicMeshIndexPolicy::NumPoolBuckets - 1))
			{
				GDynamicMeshIndexPool.ReleasePooledResource(MoveTemp(IndexBufferRHI));
			}

			IndexBufferRHI = nullptr;
		}
	}

	virtual FBufferRHIRef AllocVertexBuffer(FRHICommandListBase& RHICmdList, uint32 Stride, uint32 NumElements)
	{
		uint32 SizeInBytes = NumElements * Stride;
		if (SizeInBytes <= FGlobalDynamicMeshVertexPolicy().GetPoolBucketSize(FGlobalDynamicMeshVertexPolicy::NumPoolBuckets - 1))
		{
			return GDynamicMeshVertexPool.CreatePooledResource(RHICmdList, SizeInBytes);
		}

		return FDynamicMeshBufferAllocator::AllocVertexBuffer(RHICmdList, Stride, NumElements);
	}

	virtual void ReleaseVertexBuffer(FBufferRHIRef& VertexBufferRHI)
	{
		if (IsValidRef(VertexBufferRHI))
		{
			if (VertexBufferRHI->GetSize() <= FGlobalDynamicMeshVertexPolicy().GetPoolBucketSize(FGlobalDynamicMeshVertexPolicy::NumPoolBuckets - 1))
			{
				GDynamicMeshVertexPool.ReleasePooledResource(MoveTemp(VertexBufferRHI));
			}

			VertexBufferRHI = nullptr;
		}
	}
};

static FPooledDynamicMeshBufferAllocator DefaultDynamicMeshBufferAllocator;

/** The index buffer type used for dynamic meshes. */
class FPooledDynamicMeshIndexBuffer : public FDynamicPrimitiveResource, public FIndexBuffer
{
public:
	TArray<DynamicMeshIndexType> Indices;

	FPooledDynamicMeshIndexBuffer(FDynamicMeshBufferAllocator& InDynamicMeshBufferAllocator)
		: DynamicMeshBufferAllocator(InDynamicMeshBufferAllocator)
	{
	}

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPooledDynamicMeshIndexBuffer::InitRHI)

		IndexBufferRHI = DynamicMeshBufferAllocator.AllocIndexBuffer(RHICmdList, Indices.Num());

		// Write the indices to the index buffer.
		void* Buffer;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RHILockBuffer)
			Buffer = RHICmdList.LockBuffer(IndexBufferRHI,0,Indices.Num() * sizeof(DynamicMeshIndexType),RLM_WriteOnly);
		}
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Memcpy)
			FMemory::Memcpy(Buffer, Indices.GetData(),Indices.Num() * sizeof(DynamicMeshIndexType));
		}
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RHIUnlockBuffer)
			RHICmdList.UnlockBuffer(IndexBufferRHI);
		}
	}
	
	virtual void ReleaseRHI() override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPooledDynamicMeshIndexBuffer::ReleaseRHI)

		DynamicMeshBufferAllocator.ReleaseIndexBuffer(IndexBufferRHI);
		FIndexBuffer::ReleaseRHI();
	}

	// FDynamicPrimitiveResource interface.
	virtual void InitPrimitiveResource(FRHICommandListBase& RHICmdList) override
	{
		InitResource(RHICmdList);
	}
	virtual void ReleasePrimitiveResource() override
	{
		ReleaseResource();
		delete this;
	}

private:
	FDynamicMeshBufferAllocator& DynamicMeshBufferAllocator;
};

/** The vertex buffer type used for dynamic meshes. */
class FPooledDynamicMeshVertexBuffer : public FDynamicPrimitiveResource, public FRenderResource
{
public:
	FVertexBuffer PositionBuffer;
	FVertexBuffer TangentBuffer;
	FVertexBuffer TexCoordBuffer;
	FVertexBuffer ColorBuffer;

	FShaderResourceViewRHIRef TangentBufferSRV;
	FShaderResourceViewRHIRef TexCoordBufferSRV;
	FShaderResourceViewRHIRef ColorBufferSRV;
	FShaderResourceViewRHIRef PositionBufferSRV;

	TArray<FDynamicMeshVertex> Vertices;

	FPooledDynamicMeshVertexBuffer(ERHIFeatureLevel::Type InFeatureLevel, uint32 InNumTexCoords, uint32 InLightmapCoordinateIndex, bool InUse16bitTexCoord, FDynamicMeshBufferAllocator& InDynamicMeshBufferAllocator)
		: NumTexCoords(InNumTexCoords)
		, LightmapCoordinateIndex(InLightmapCoordinateIndex)
		, Use16bitTexCoord(InUse16bitTexCoord)
		, FeatureLevel(InFeatureLevel)
		, DynamicMeshBufferAllocator(InDynamicMeshBufferAllocator)
	{
		check(NumTexCoords > 0 && NumTexCoords <= MAX_STATIC_TEXCOORDS);
		check(LightmapCoordinateIndex < NumTexCoords);
	}

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPooledDynamicMeshVertexBuffer::InitRHI)

		uint32 TextureStride = sizeof(FVector2f);
		EPixelFormat TextureFormat = PF_G32R32F;

		if (Use16bitTexCoord)
		{
			TextureStride = sizeof(FVector2DHalf);
			TextureFormat = PF_G16R16F;
		}

		PositionBuffer.VertexBufferRHI = DynamicMeshBufferAllocator.AllocVertexBuffer(RHICmdList, sizeof(FVector3f), Vertices.Num());
		TangentBuffer.VertexBufferRHI  = DynamicMeshBufferAllocator.AllocVertexBuffer(RHICmdList, sizeof(FPackedNormal), 2 * Vertices.Num());
		TexCoordBuffer.VertexBufferRHI = DynamicMeshBufferAllocator.AllocVertexBuffer(RHICmdList, TextureStride, NumTexCoords * Vertices.Num());
		ColorBuffer.VertexBufferRHI    = DynamicMeshBufferAllocator.AllocVertexBuffer(RHICmdList, sizeof(FColor), Vertices.Num());

		void* TexCoordBufferData = RHICmdList.LockBuffer(TexCoordBuffer.VertexBufferRHI, 0, NumTexCoords * TextureStride * Vertices.Num(), RLM_WriteOnly);
		FVector2f* TexCoordBufferData32 = !Use16bitTexCoord ? static_cast<FVector2f*>(TexCoordBufferData) : nullptr;
		FVector2DHalf* TexCoordBufferData16 = Use16bitTexCoord ? static_cast<FVector2DHalf*>(TexCoordBufferData) : nullptr;

		// Copy the vertex data into the vertex buffers.
		FVector3f* PositionBufferData			= static_cast<FVector3f*>(RHICmdList.LockBuffer(PositionBuffer.VertexBufferRHI, 0, sizeof(FVector3f) * Vertices.Num(), RLM_WriteOnly));
		FPackedNormal* TangentBufferData	= static_cast<FPackedNormal*>(RHICmdList.LockBuffer(TangentBuffer.VertexBufferRHI, 0, 2 * sizeof(FPackedNormal) * Vertices.Num(), RLM_WriteOnly));	
		FColor* ColorBufferData				= static_cast<FColor*>(RHICmdList.LockBuffer(ColorBuffer.VertexBufferRHI, 0, sizeof(FColor) * Vertices.Num(), RLM_WriteOnly));

		{
			// This code will generate a lot of page faults when the memory has never been written to
			// so we'll know pooled buffers are not behaving optimally if this shows up more than we would expect.
			TRACE_CPUPROFILER_EVENT_SCOPE(CopyDataToVertexBuffers)

			for (int32 i = 0; i < Vertices.Num(); i++)
			{
				PositionBufferData[i] = Vertices[i].Position;
				TangentBufferData[2 * i + 0] = Vertices[i].TangentX;
				TangentBufferData[2 * i + 1] = Vertices[i].TangentZ;
				ColorBufferData[i] = Vertices[i].Color;

				for (uint32 j = 0; j < NumTexCoords; j++)
				{
					if (Use16bitTexCoord)
					{
						TexCoordBufferData16[NumTexCoords * i + j] = FVector2DHalf(Vertices[i].TextureCoordinate[j]);
					}
					else
					{
						TexCoordBufferData32[NumTexCoords * i + j] = Vertices[i].TextureCoordinate[j];
					}
				}
			}
		}

		RHICmdList.UnlockBuffer(PositionBuffer.VertexBufferRHI);
		RHICmdList.UnlockBuffer(TangentBuffer.VertexBufferRHI);
		RHICmdList.UnlockBuffer(TexCoordBuffer.VertexBufferRHI);
		RHICmdList.UnlockBuffer(ColorBuffer.VertexBufferRHI);

		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			TangentBufferSRV = RHICmdList.CreateShaderResourceView(
				TangentBuffer.VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8G8B8A8_SNORM));
			TexCoordBufferSRV = RHICmdList.CreateShaderResourceView(
				TexCoordBuffer.VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(TextureFormat));
			ColorBufferSRV = RHICmdList.CreateShaderResourceView(
				ColorBuffer.VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8G8B8A8));

			PositionBufferSRV = RHICmdList.CreateShaderResourceView(
				PositionBuffer.VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_FLOAT));
		}
	}

	void InitResource(FRHICommandListBase& RHICmdList) override
	{
		FRenderResource::InitResource(RHICmdList);
		PositionBuffer.InitResource(RHICmdList);
		TangentBuffer.InitResource(RHICmdList);
		TexCoordBuffer.InitResource(RHICmdList);
		ColorBuffer.InitResource(RHICmdList);
	}

	void ReleaseResource() override
	{
		FRenderResource::ReleaseResource();
		PositionBuffer.ReleaseResource();
		TangentBuffer.ReleaseResource();
		TexCoordBuffer.ReleaseResource();
		ColorBuffer.ReleaseResource();
	}

	virtual void ReleaseRHI() override
	{
		DynamicMeshBufferAllocator.ReleaseVertexBuffer(PositionBuffer.VertexBufferRHI);
		DynamicMeshBufferAllocator.ReleaseVertexBuffer(TangentBuffer.VertexBufferRHI);
		DynamicMeshBufferAllocator.ReleaseVertexBuffer(TexCoordBuffer.VertexBufferRHI);
		DynamicMeshBufferAllocator.ReleaseVertexBuffer(ColorBuffer.VertexBufferRHI);
	}

	// FDynamicPrimitiveResource interface.
	virtual void InitPrimitiveResource(FRHICommandListBase& RHICmdList) override
	{
		InitResource(RHICmdList);
	}

	virtual void ReleasePrimitiveResource() override
	{
		ReleaseResource();
		delete this;
	}

	const uint32 GetNumTexCoords() const
	{
		return NumTexCoords;
	}

	const uint32 GetLightmapCoordinateIndex() const
	{
		return LightmapCoordinateIndex;
	}

	const bool GetUse16bitTexCoords() const
	{
		return Use16bitTexCoord;
	}
private:
	const uint32 NumTexCoords;
	const uint32 LightmapCoordinateIndex;
	const bool Use16bitTexCoord;
	const ERHIFeatureLevel::Type FeatureLevel;
	FDynamicMeshBufferAllocator& DynamicMeshBufferAllocator;
};

/** The vertex factory type used for dynamic meshes. */
class FPooledDynamicMeshVertexFactory : public FDynamicPrimitiveResource, public FLocalVertexFactory
{
public:

	/** Initialization constructor. */
	FPooledDynamicMeshVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, const FPooledDynamicMeshVertexBuffer* InVertexBuffer) : FLocalVertexFactory(InFeatureLevel, "FPooledDynamicMeshVertexFactory"), VertexBuffer(InVertexBuffer) {}

	void InitResource(FRHICommandListBase& RHICmdList) override
	{
		FLocalVertexFactory* VertexFactory = this;
		const FPooledDynamicMeshVertexBuffer* PooledVertexBuffer = VertexBuffer;

		FDataType LocalData;
		LocalData.PositionComponent = FVertexStreamComponent(
			&PooledVertexBuffer->PositionBuffer,
			0,
			sizeof(FVector3f),
			VET_Float3
		);

		LocalData.NumTexCoords = PooledVertexBuffer->GetNumTexCoords();
		{
			LocalData.LightMapCoordinateIndex = PooledVertexBuffer->GetLightmapCoordinateIndex();
			LocalData.TangentsSRV = PooledVertexBuffer->TangentBufferSRV;
			LocalData.TextureCoordinatesSRV = PooledVertexBuffer->TexCoordBufferSRV;
			LocalData.ColorComponentsSRV = PooledVertexBuffer->ColorBufferSRV;
			LocalData.PositionComponentSRV = PooledVertexBuffer->PositionBufferSRV;
		}

		{
			EVertexElementType UVDoubleWideVertexElementType = VET_None;
			EVertexElementType UVVertexElementType = VET_None;
			uint32 UVSizeInBytes = 0;
			if (PooledVertexBuffer->GetUse16bitTexCoords())
			{
				UVSizeInBytes = sizeof(FVector2DHalf);
				UVDoubleWideVertexElementType = VET_Half4;
				UVVertexElementType = VET_Half2;
			}
			else
			{
				UVSizeInBytes = sizeof(FVector2f);
				UVDoubleWideVertexElementType = VET_Float4;
				UVVertexElementType = VET_Float2;
			}

			int32 UVIndex;
			uint32 UvStride = UVSizeInBytes * PooledVertexBuffer->GetNumTexCoords();
			for (UVIndex = 0; UVIndex < (int32)PooledVertexBuffer->GetNumTexCoords() - 1; UVIndex += 2)
			{
				LocalData.TextureCoordinates.Add
				(
					FVertexStreamComponent(
						&PooledVertexBuffer->TexCoordBuffer, 
						UVSizeInBytes * UVIndex, 
						UvStride,
						UVDoubleWideVertexElementType,
						EVertexStreamUsage::ManualFetch
					)
				);
			}

			// possible last UV channel if we have an odd number
			if (UVIndex < (int32)PooledVertexBuffer->GetNumTexCoords())
			{
				LocalData.TextureCoordinates.Add(FVertexStreamComponent(
					&PooledVertexBuffer->TexCoordBuffer,
					UVSizeInBytes * UVIndex,
					UvStride,
					UVVertexElementType, 
					EVertexStreamUsage::ManualFetch
				));
			}

			LocalData.TangentBasisComponents[0] = FVertexStreamComponent(&PooledVertexBuffer->TangentBuffer, 0, 2 * sizeof(FPackedNormal), VET_PackedNormal, EVertexStreamUsage::ManualFetch);
			LocalData.TangentBasisComponents[1] = FVertexStreamComponent(&PooledVertexBuffer->TangentBuffer, sizeof(FPackedNormal), 2 * sizeof(FPackedNormal), VET_PackedNormal, EVertexStreamUsage::ManualFetch);
			LocalData.ColorComponent = FVertexStreamComponent(&PooledVertexBuffer->ColorBuffer, 0, sizeof(FColor), VET_Color, EVertexStreamUsage::ManualFetch);
		}
		VertexFactory->SetData(RHICmdList, LocalData);

		FLocalVertexFactory::InitResource(RHICmdList);
	}

	// FDynamicPrimitiveResource interface.
	void InitPrimitiveResource(FRHICommandListBase& RHICmdList) override
	{
		InitResource(RHICmdList);
	}

	void ReleasePrimitiveResource() override
	{
		ReleaseResource();
		delete this;
	}

private:
	const FPooledDynamicMeshVertexBuffer* VertexBuffer;

};

/** The primitive uniform buffer used for dynamic meshes. */
class FDynamicMeshPrimitiveUniformBuffer : public FDynamicPrimitiveResource, public TUniformBuffer<FPrimitiveUniformShaderParameters>
{
public:
	
	// FDynamicPrimitiveResource interface.
	virtual void InitPrimitiveResource(FRHICommandListBase& RHICmdList)
	{
		InitResource(RHICmdList);
	}
	virtual void ReleasePrimitiveResource()
	{
		ReleaseResource();
		delete this;
	}
};

FDynamicMeshBuilder::FDynamicMeshBuilder(ERHIFeatureLevel::Type InFeatureLevel, uint32 InNumTexCoords, uint32 InLightmapCoordinateIndex, bool InUse16bitTexCoord, FDynamicMeshBufferAllocator* InDynamicMeshBufferAllocator)
	: FeatureLevel(InFeatureLevel)
{
	OneFrameResources = nullptr;

	if (InDynamicMeshBufferAllocator == nullptr)
	{
		InDynamicMeshBufferAllocator = &DefaultDynamicMeshBufferAllocator;
	}

	VertexBuffer = new FPooledDynamicMeshVertexBuffer(InFeatureLevel, InNumTexCoords, InLightmapCoordinateIndex, InUse16bitTexCoord, *InDynamicMeshBufferAllocator);
	IndexBuffer = new FPooledDynamicMeshIndexBuffer(*InDynamicMeshBufferAllocator);
}

FDynamicMeshBuilder::~FDynamicMeshBuilder()
{
	//Delete the resources if they have not been already. At this point they are only valid if Draw() has never been called,
	//so the resources have not been passed to the rendering thread.  Also they do not need to be released,
	//since they are only initialized when Draw() is called.
	delete VertexBuffer;
	delete IndexBuffer;
}

int32 FDynamicMeshBuilder::AddVertex(
	const FVector3f& InPosition,
	const FVector2f& InTextureCoordinate,
	const FVector3f& InTangentX,
	const FVector3f& InTangentY,
	const FVector3f& InTangentZ,
	const FColor& InColor
	)
{
	int32 VertexIndex = VertexBuffer->Vertices.Num();
	FDynamicMeshVertex* Vertex = new(VertexBuffer->Vertices) FDynamicMeshVertex;
	Vertex->Position = InPosition;
	Vertex->TextureCoordinate[0] = InTextureCoordinate;
	Vertex->TangentX = InTangentX;
	Vertex->TangentZ = InTangentZ;
	Vertex->TangentZ.Vector.W = GetBasisDeterminantSignByte( InTangentX, InTangentY, InTangentZ );
	Vertex->Color = InColor;

	return VertexIndex;
}

/** Adds a vertex to the mesh. */
int32 FDynamicMeshBuilder::AddVertex(const FDynamicMeshVertex &InVertex)
{
	int32 VertexIndex = VertexBuffer->Vertices.Num();
	FDynamicMeshVertex* Vertex = new(VertexBuffer->Vertices) FDynamicMeshVertex(InVertex);

	return VertexIndex;
}

/** Adds a triangle to the mesh. */
void FDynamicMeshBuilder::AddTriangle(int32 V0,int32 V1,int32 V2)
{
	IndexBuffer->Indices.Add(static_cast<DynamicMeshIndexType>(V0));
	IndexBuffer->Indices.Add(static_cast<DynamicMeshIndexType>(V1));
	IndexBuffer->Indices.Add(static_cast<DynamicMeshIndexType>(V2));
}

/** Adds many vertices to the mesh. Returns start index of verts in the overall array. */
int32 FDynamicMeshBuilder::AddVertices(const TArray<FDynamicMeshVertex> &InVertices)
{
	int32 StartIndex = VertexBuffer->Vertices.Num();
	VertexBuffer->Vertices.Append(InVertices);
	return StartIndex;
}

/** Add many indices to the mesh. */
void FDynamicMeshBuilder::AddTriangles(const TArray<uint32> &InIndices)
{
	if (IndexBuffer->Indices.GetTypeSize() == InIndices.GetTypeSize())
	{
		IndexBuffer->Indices.Append(InIndices);
	}
	else
	{
		IndexBuffer->Indices.Reserve(IndexBuffer->Indices.Num() + InIndices.Num());
		for (int32 Index : InIndices)
		{
			IndexBuffer->Indices.Add(static_cast<DynamicMeshIndexType>(Index));
		}
	}
}

/** Pre-allocate space for the given number of vertices. */
void FDynamicMeshBuilder::ReserveVertices(int32 InNumVertices)
{
	VertexBuffer->Vertices.Reserve(InNumVertices);
}

/** Pre-allocate space for the given number of triangles. */
void FDynamicMeshBuilder::ReserveTriangles(int32 InNumTriangles)
{
	IndexBuffer->Indices.Reserve(3 * InNumTriangles);
}

FMeshBuilderOneFrameResources::~FMeshBuilderOneFrameResources()
{
	if (VertexBuffer)
	{
		VertexBuffer->ReleaseResource();
		delete VertexBuffer;
	}

	if (IndexBuffer)
	{
		if (IndexBuffer->Indices.Num())
		{
			IndexBuffer->ReleaseResource();
		}
		delete IndexBuffer;
	}

	if (VertexFactory)
	{
		VertexFactory->ReleaseResource();
		delete VertexFactory;
	}

	if (PrimitiveUniformBuffer)
	{
		PrimitiveUniformBuffer->ReleaseResource();
		delete PrimitiveUniformBuffer;
	}	
}

void FDynamicMeshBuilder::GetMesh(const FMatrix& LocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, bool bReceivesDecals, int32 ViewIndex, FMeshElementCollector& Collector)
{
	GetMesh(LocalToWorld, MaterialRenderProxy, DepthPriorityGroup, bDisableBackfaceCulling, bReceivesDecals, true, ViewIndex, Collector, NULL);
}

void FDynamicMeshBuilder::GetMesh(const FMatrix& LocalToWorld,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriorityGroup,bool bDisableBackfaceCulling, bool bReceivesDecals, bool bUseSelectionOutline, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	GetMesh(LocalToWorld, MaterialRenderProxy, DepthPriorityGroup, bDisableBackfaceCulling, bReceivesDecals, true, ViewIndex, Collector, HitProxy != nullptr ? HitProxy->Id : FHitProxyId());
}

void FDynamicMeshBuilder::GetMesh(const FMatrix& LocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, bool bReceivesDecals, bool bUseSelectionOutline, int32 ViewIndex, FMeshElementCollector& Collector, const FHitProxyId HitProxyId)
{
	FDynamicMeshBuilderSettings Settings;
	Settings.bDisableBackfaceCulling = bDisableBackfaceCulling;
	Settings.bReceivesDecals = bReceivesDecals;
	Settings.bUseSelectionOutline = bUseSelectionOutline;
	GetMesh(LocalToWorld, MaterialRenderProxy, DepthPriorityGroup, Settings, nullptr, ViewIndex, Collector, HitProxyId);
}

void FDynamicMeshBuilder::GetMesh(
	const FMatrix& LocalToWorld,
	const FMaterialRenderProxy* MaterialRenderProxy,
	uint8 DepthPriorityGroup,
	const FDynamicMeshBuilderSettings& Settings,
	FDynamicMeshDrawOffset const * const DrawOffset,
	int32 ViewIndex,
	FMeshElementCollector& Collector,
	const FHitProxyId HitProxyId)
{
	GetMesh(LocalToWorld, LocalToWorld, MaterialRenderProxy, DepthPriorityGroup, Settings, DrawOffset, ViewIndex, Collector, HitProxyId);
}

void FDynamicMeshBuilder::GetMesh(
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FMaterialRenderProxy* MaterialRenderProxy,
	uint8 DepthPriorityGroup,
	const FDynamicMeshBuilderSettings& Settings,
	FDynamicMeshDrawOffset const * const DrawOffset,
	int32 ViewIndex,
	FMeshElementCollector& Collector,
	const FHitProxyId HitProxyId )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicMeshBuilder::GetMesh)

	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

	// Only draw non-empty meshes.
	if((VertexBuffer && VertexBuffer->Vertices.Num() > 0) || (DrawOffset != nullptr))
	{
		if (VertexBuffer || IndexBuffer)
		{
			check(VertexBuffer || IndexBuffer);

			OneFrameResources = &Collector.AllocateOneFrameResource<FMeshBuilderOneFrameResources>();

			OneFrameResources->VertexBuffer = VertexBuffer;
			OneFrameResources->IndexBuffer = IndexBuffer;

			if (OneFrameResources->VertexBuffer)
			{
				OneFrameResources->VertexBuffer->InitResource(RHICmdList);
			}
			if (OneFrameResources->IndexBuffer && IndexBuffer->Indices.Num())
			{
				OneFrameResources->IndexBuffer->InitResource(RHICmdList);
			}
			OneFrameResources->VertexFactory = new FPooledDynamicMeshVertexFactory(FeatureLevel, VertexBuffer);
			OneFrameResources->VertexFactory->InitResource(RHICmdList);

			// Create the primitive uniform buffer.
			OneFrameResources->PrimitiveUniformBuffer = new FDynamicMeshPrimitiveUniformBuffer();
			FPrimitiveUniformShaderParameters PrimitiveParams = FPrimitiveUniformShaderParametersBuilder{}
				.Defaults()
					.LocalToWorld(LocalToWorld)
					.PreviousLocalToWorld(PreviousLocalToWorld)
					.ActorWorldPosition(LocalToWorld.GetOrigin())
					.WorldBounds(FBoxSphereBounds(EForceInit::ForceInit))
					.LocalBounds(FBoxSphereBounds(EForceInit::ForceInit))
					.ReceivesDecals(Settings.bReceivesDecals)
					.OutputVelocity(true)
					.IsFirstPerson(Settings.bIsFirstPerson)
				.Build();

			
			OneFrameResources->PrimitiveUniformBuffer->SetContents(RHICmdList, PrimitiveParams);

			OneFrameResources->PrimitiveUniformBuffer->InitResource(RHICmdList);

			// Clear the resource pointers so they cannot be overwritten accidentally.
			// These resources will be released by the PDI.
			VertexBuffer = nullptr;
			IndexBuffer = nullptr;
		}

		const bool bHasValidIndexBuffer = OneFrameResources->IndexBuffer && OneFrameResources->IndexBuffer->Indices.Num();
		const bool bHasValidVertexBuffer = OneFrameResources->VertexBuffer && OneFrameResources->VertexBuffer->Vertices.Num();
		// Draw the mesh.
		FMeshBatch& Mesh = Collector.AllocateMesh();
		FMeshBatchElement& BatchElement = Mesh.Elements[0];
		BatchElement.IndexBuffer = bHasValidIndexBuffer ? OneFrameResources->IndexBuffer : nullptr;
		Mesh.VertexFactory = OneFrameResources->VertexFactory;
		Mesh.MaterialRenderProxy = MaterialRenderProxy;
		BatchElement.PrimitiveUniformBufferResource = OneFrameResources->PrimitiveUniformBuffer;

		Mesh.CastShadow = Settings.CastShadow;
		Mesh.bWireframe = Settings.bWireframe;
		Mesh.bCanApplyViewModeOverrides = Settings.bCanApplyViewModeOverrides;
		Mesh.bUseWireframeSelectionColoring = Settings.bUseWireframeSelectionColoring;
		Mesh.ReverseCulling = LocalToWorld.Determinant() < 0.0f ? true : false;
		Mesh.bDisableBackfaceCulling = Settings.bDisableBackfaceCulling;
		Mesh.Type = PT_TriangleList;
		Mesh.DepthPriorityGroup = DepthPriorityGroup;
		Mesh.bUseSelectionOutline = Settings.bUseSelectionOutline;
		Mesh.BatchHitProxyId = HitProxyId;

		BatchElement.FirstIndex = DrawOffset ? DrawOffset->FirstIndex : 0;
		BatchElement.NumPrimitives = DrawOffset ? DrawOffset->NumPrimitives : (bHasValidIndexBuffer ? (OneFrameResources->IndexBuffer->Indices.Num() / 3) : (bHasValidVertexBuffer ? OneFrameResources->VertexBuffer->Vertices.Num() / 3 : 0));
		BatchElement.MinVertexIndex = DrawOffset ? DrawOffset->MinVertexIndex : 0;
		BatchElement.MaxVertexIndex = DrawOffset ? DrawOffset->MaxVertexIndex : (bHasValidVertexBuffer ? OneFrameResources->VertexBuffer->Vertices.Num() - 1 : 0);

		Collector.AddMesh(ViewIndex, Mesh);
	}
}

void FDynamicMeshBuilder::GetMeshElement(const FMatrix& LocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, bool bReceivesDecals, int32 ViewIndex, FMeshBuilderOneFrameResources& OneFrameResource, FMeshBatch& Mesh)
{
	FPrimitiveUniformShaderParameters PrimitiveParams = FPrimitiveUniformShaderParametersBuilder{}
		.Defaults()
			.LocalToWorld(LocalToWorld)
			.ActorWorldPosition(LocalToWorld.GetOrigin())
			.WorldBounds(FBoxSphereBounds(EForceInit::ForceInit))
			.LocalBounds(FBoxSphereBounds(EForceInit::ForceInit))
			.ReceivesDecals(bReceivesDecals)
			.OutputVelocity(true)
		.Build();

	GetMeshElement(PrimitiveParams, MaterialRenderProxy, DepthPriorityGroup, bDisableBackfaceCulling, ViewIndex, OneFrameResource, Mesh);
}

void FDynamicMeshBuilder::GetMeshElement(const FPrimitiveUniformShaderParameters& PrimitiveParams, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, int32 ViewIndex, FMeshBuilderOneFrameResources& OneFrameResource, FMeshBatch& Mesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicMeshBuilder::GetMeshElement)

	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	// Only draw non-empty meshes.
	if (VertexBuffer->Vertices.Num() > 0)
	{
		if (VertexBuffer || IndexBuffer)
		{
			OneFrameResource.VertexBuffer = VertexBuffer;
			OneFrameResource.IndexBuffer = IndexBuffer;

			if (OneFrameResource.VertexBuffer)
			{
				OneFrameResource.VertexBuffer->InitResource(RHICmdList);
			}
			if (OneFrameResource.IndexBuffer && IndexBuffer->Indices.Num())
			{
				OneFrameResource.IndexBuffer->InitResource(RHICmdList);
			}

			OneFrameResource.VertexFactory = new FPooledDynamicMeshVertexFactory(FeatureLevel, VertexBuffer);
			OneFrameResource.VertexFactory->InitResource(RHICmdList);

			// Create the primitive uniform buffer.
			OneFrameResource.PrimitiveUniformBuffer = new FDynamicMeshPrimitiveUniformBuffer();

			OneFrameResource.PrimitiveUniformBuffer->SetContents(RHICmdList, PrimitiveParams);

			OneFrameResource.PrimitiveUniformBuffer->InitResource(RHICmdList);

			// Clear the resource pointers so they cannot be overwritten accidentally.
			// These resources will be released by the PDI.
			VertexBuffer = nullptr;
			IndexBuffer = nullptr;
		}

		const bool bHasValidIndexBuffer = OneFrameResource.IndexBuffer && OneFrameResource.IndexBuffer->Indices.Num();
		const bool bHasValidVertexBuffer = OneFrameResource.VertexBuffer && OneFrameResource.VertexBuffer->Vertices.Num();
		// Draw the mesh.
		FMeshBatchElement& BatchElement = Mesh.Elements[0];
		BatchElement.IndexBuffer = bHasValidIndexBuffer ? OneFrameResource.IndexBuffer : nullptr;
		Mesh.VertexFactory = OneFrameResource.VertexFactory;
		Mesh.MaterialRenderProxy = MaterialRenderProxy;
		BatchElement.PrimitiveUniformBufferResource = OneFrameResource.PrimitiveUniformBuffer;
		// previous l2w not used so treat as static
		BatchElement.FirstIndex = 0;
		BatchElement.NumPrimitives = bHasValidIndexBuffer ? (OneFrameResource.IndexBuffer->Indices.Num() / 3) : (bHasValidVertexBuffer ? OneFrameResource.VertexBuffer->Vertices.Num() / 3 : 0);
		BatchElement.MinVertexIndex = 0;
		BatchElement.MaxVertexIndex = bHasValidVertexBuffer ? OneFrameResource.VertexBuffer->Vertices.Num() - 1 : 0;
		Mesh.ReverseCulling = PrimitiveParams.LocalToRelativeWorld.Determinant() < 0.0f ? true : false;
		Mesh.bDisableBackfaceCulling = bDisableBackfaceCulling;
		Mesh.Type = PT_TriangleList;
		Mesh.DepthPriorityGroup = DepthPriorityGroup;
		Mesh.bUseSelectionOutline = false;
		Mesh.BatchHitProxyId = FHitProxyId();
	}
}

void FDynamicMeshBuilder::Draw(FPrimitiveDrawInterface* PDI,const FMatrix& LocalToWorld,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriorityGroup,bool bDisableBackfaceCulling, bool bReceivesDecals, const FHitProxyId HitProxyId )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicMeshBuilder::Draw)

	// Only draw non-empty meshes.
	if(VertexBuffer->Vertices.Num() > 0 && IndexBuffer->Indices.Num() > 0)
	{
		// Register the dynamic resources with the PDI.
		PDI->RegisterDynamicResource(VertexBuffer);

		if (IndexBuffer && IndexBuffer->Indices.Num())
		{
			PDI->RegisterDynamicResource(IndexBuffer);
		}

		// Create the vertex factory.
		FPooledDynamicMeshVertexFactory* VertexFactory = new FPooledDynamicMeshVertexFactory(FeatureLevel, VertexBuffer);
		PDI->RegisterDynamicResource(VertexFactory);

		// Create the primitive uniform buffer.
		FDynamicMeshPrimitiveUniformBuffer* PrimitiveUniformBuffer = new FDynamicMeshPrimitiveUniformBuffer();
		FPrimitiveUniformShaderParameters PrimitiveParams = FPrimitiveUniformShaderParametersBuilder{}
			.Defaults()
				.LocalToWorld(LocalToWorld)
				.ActorWorldPosition(LocalToWorld.GetOrigin())
				.WorldBounds(FBoxSphereBounds(EForceInit::ForceInit))
				.LocalBounds(FBoxSphereBounds(EForceInit::ForceInit))
				.ReceivesDecals(bReceivesDecals)
				.OutputVelocity(true)
			.Build();

		if (IsInGameThread())
		{
			BeginSetUniformBufferContents(*PrimitiveUniformBuffer, PrimitiveParams);
		}
		else
		{
			PrimitiveUniformBuffer->SetContents(FRHICommandListImmediate::Get(), PrimitiveParams);
		}
		PDI->RegisterDynamicResource(PrimitiveUniformBuffer);

		const bool HasValidIndexBuffer = IndexBuffer && IndexBuffer->Indices.Num();
		// Draw the mesh.
		FMeshBatch Mesh;
		FMeshBatchElement& BatchElement = Mesh.Elements[0];
		BatchElement.IndexBuffer = HasValidIndexBuffer ? IndexBuffer : nullptr;
		Mesh.VertexFactory = VertexFactory;
		Mesh.MaterialRenderProxy = MaterialRenderProxy;
		BatchElement.PrimitiveUniformBufferResource = PrimitiveUniformBuffer;
		// previous l2w not used so treat as static
		BatchElement.FirstIndex = 0;
		BatchElement.NumPrimitives = HasValidIndexBuffer ? (IndexBuffer->Indices.Num() / 3) : (VertexBuffer->Vertices.Num() / 3);
		BatchElement.MinVertexIndex = 0;
		BatchElement.MaxVertexIndex = VertexBuffer->Vertices.Num() - 1;
		Mesh.ReverseCulling = LocalToWorld.Determinant() < 0.0f ? true : false;
		Mesh.bDisableBackfaceCulling = bDisableBackfaceCulling;
		Mesh.Type = PT_TriangleList;
		Mesh.DepthPriorityGroup = DepthPriorityGroup;
		Mesh.BatchHitProxyId = HitProxyId;
		PDI->DrawMesh(Mesh);

		// Clear the resource pointers so they cannot be overwritten accidentally.
		// These resources will be released by the PDI.
		VertexBuffer = nullptr;
		IndexBuffer = nullptr;
	}
}
