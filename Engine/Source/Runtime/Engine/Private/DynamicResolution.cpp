// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicResolutionProxy.h"
#include "DynamicResolutionState.h"

#include "Engine/Engine.h"
#include "Misc/App.h"
#include "RenderingThread.h"
#include "RenderTimer.h"
#include "SceneView.h"
#include "Stats/StatsTrace.h"


static TAutoConsoleVariable<float> CVarDynamicResMinSP(
	TEXT("r.DynamicRes.MinScreenPercentage"),
	DynamicRenderScaling::FractionToPercentage(DynamicRenderScaling::FHeuristicSettings::kDefaultMinResolutionFraction),
	TEXT("Minimal primary screen percentage."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarDynamicResMaxSP(
	TEXT("r.DynamicRes.MaxScreenPercentage"),
	DynamicRenderScaling::FractionToPercentage(DynamicRenderScaling::FHeuristicSettings::kDefaultMaxResolutionFraction),
	TEXT("Maximal primary screen percentage. Importantly this setting controls the preallocated video memory needed by the renderer to render."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarDynamicResThrottlingMaxSP(
	TEXT("r.DynamicRes.ThrottlingMaxScreenPercentage"),
	DynamicRenderScaling::FractionToPercentage(DynamicRenderScaling::FHeuristicSettings::kDefaultThrottlingMaxResolutionFraction),
	TEXT("Throttle the primary screen percentage allowed by the heuristic to this max value when enabled. This has no effect on preallocated video memory.\n")
	TEXT("This is for instance useful when the video game wants to trottle power consumption when inactive without resizing internal renderer's render targets\n")
	TEXT("(which can result in popping)"),
	ECVF_Default);

// TODO: Seriously need a centralized engine perf manager.
static TAutoConsoleVariable<float> CVarFrameTimeBudget(
	TEXT("r.DynamicRes.FrameTimeBudget"),
	33.3f,
	TEXT("Frame's time budget in milliseconds."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<int32> CVarUseGameThreadCriticalPath(
	TEXT("r.DynamicRes.UseGameThreadCriticalPath"), 0,
	TEXT("Whether to use game thread critical path time when determining whether game is CPU bound."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarTargetedGPUHeadRoomPercentage(
	TEXT("r.DynamicRes.TargetedGPUHeadRoomPercentage"),
	10.0f,
	TEXT("Targeted GPU headroom (in percent from r.DynamicRes.FrameTimeBudget)."),
	ECVF_RenderThreadSafe | ECVF_Default);

/** On desktop, the swap chain doesn't allow tear amount configuration, so an overbudget frame can be droped with r.VSync=1.
 * So need to lower the heuristic's target budget to lower chances to go overbudget.
 *
 * Moreover the GPU is a shared ressource with other process which may or may not be included in our GPU timings,
 * and need to leave some GPU capacity to these application to not get preempted by OS scheduler.
 * Given we can measure other application's GPU cost, need to leave enough headroom for them all the time.
 */
static TAutoConsoleVariable<float> CVarOverBudgetGPUHeadRoomPercentage(
	TEXT("r.DynamicRes.OverBudgetGPUHeadRoomPercentage"),
	0.0f,
	TEXT("Amount of GPU headroom needed from which the frame is considered over budget. This is for platform not supporting controllable tearing with VSync (in percent from r.DynamicRes.FrameTimeBudget)."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<int32> CVarHistorySize(
	TEXT("r.DynamicRes.HistorySize"),
	16,
	TEXT("Number of frames keept in the history."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarFrameWeightExponent(
	TEXT("r.DynamicRes.FrameWeightExponent"),
	0.9f,
	TEXT("Recursive weight of frame N-1 against frame N."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<int32> CVarFrameChangePeriod(
	TEXT("r.DynamicRes.MinResolutionChangePeriod"),
	8,
	TEXT("Minimal number of frames between resolution changes, important to avoid input ")
	TEXT("sample position interferences in TAA upsample."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarIncreaseAmortizationFactor(
	TEXT("r.DynamicRes.IncreaseAmortizationBlendFactor"),
	DynamicRenderScaling::FHeuristicSettings::kDefaultIncreaseAmortizationFactor,
	TEXT("Amortization blend factor when scale resolution back up to reduce resolution fraction oscillations."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarChangeThreshold(
	TEXT("r.DynamicRes.ChangePercentageThreshold"),
	DynamicRenderScaling::FractionToPercentage(DynamicRenderScaling::FHeuristicSettings::kDefaultChangeThreshold),
	TEXT("Minimal increase percentage threshold to alow when changing resolution."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<int32> CVarMaxConsecutiveOverBudgetGPUFrameCount(
	TEXT("r.DynamicRes.MaxConsecutiveOverBudgetGPUFrameCount"),
	2,
	TEXT("Maximum number of consecutive frames tolerated over GPU budget."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<int32> CVarUpperBoundQuantization(
	TEXT("r.DynamicRes.UpperBoundQuantization"),
	DynamicRenderScaling::FHeuristicSettings::kDefaultUpperBoundQuantization,
	TEXT("Quantization step count to use for upper bound screen percentage.\n")
	TEXT("If non-zero, rendertargets will be resized based on the dynamic resolution fraction, saving GPU time during clears and resolves.\n")
	TEXT("Only recommended for use with the transient allocator (on supported platforms) with a large transient texture cache (e.g RHI.TransientAllocator.TextureCacheSize=512)"),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<bool> CVarUseCPUTimeLogic(
	TEXT("r.DynamicRes.UseCPUTimeLogic"),
	false,
	TEXT("When true, enables legacy logic that checks whether the engine is game or render thread bound, and if so, allows the GPU to consider more frame history.\n")
	TEXT("When false, dynamic resolution is driven only from GPU time, and multiple over-budget GPU frames will cause a sooner drop in resolution."),
	ECVF_RenderThreadSafe | ECVF_Default);

DynamicRenderScaling::FHeuristicSettings GetPrimaryDynamicResolutionSettings()
{
	DynamicRenderScaling::FHeuristicSettings BudgetSetting;
	BudgetSetting.Model = DynamicRenderScaling::EHeuristicModel::Quadratic;
	BudgetSetting.MinResolutionFraction      = DynamicRenderScaling::GetPercentageCVarToFraction(CVarDynamicResMinSP);
	BudgetSetting.MaxResolutionFraction      = DynamicRenderScaling::GetPercentageCVarToFraction(CVarDynamicResMaxSP);
	BudgetSetting.ThrottlingMaxResolutionFraction = DynamicRenderScaling::GetPercentageCVarToFraction(CVarDynamicResThrottlingMaxSP);
	BudgetSetting.UpperBoundQuantization     = CVarUpperBoundQuantization.GetValueOnAnyThread();
	BudgetSetting.BudgetMs                   = CVarFrameTimeBudget.GetValueOnAnyThread() * (1.0f - DynamicRenderScaling::GetPercentageCVarToFraction(CVarOverBudgetGPUHeadRoomPercentage));
	BudgetSetting.ChangeThreshold            = DynamicRenderScaling::GetPercentageCVarToFraction(CVarChangeThreshold);
	BudgetSetting.TargetedHeadRoom           = DynamicRenderScaling::GetPercentageCVarToFraction(CVarTargetedGPUHeadRoomPercentage);
	BudgetSetting.IncreaseAmortizationFactor = CVarIncreaseAmortizationFactor.GetValueOnAnyThread();
	return BudgetSetting;
}

DynamicRenderScaling::FBudget GDynamicPrimaryResolutionFraction(TEXT("DynamicPrimaryResolution"), &GetPrimaryDynamicResolutionSettings);


static float TimeStampQueryResultToMilliSeconds(uint64 TimestampResult)
{
	return float(TimestampResult) / 1000.0f;
}


FDynamicResolutionHeuristicProxy::FDynamicResolutionHeuristicProxy()
{
	check(IsInGameThread());
	ResetInternal();
}

FDynamicResolutionHeuristicProxy::~FDynamicResolutionHeuristicProxy()
{
	check(IsInRenderingThread());
}

void FDynamicResolutionHeuristicProxy::Reset_RenderThread()
{
	check(IsInRenderingThread());
	ResetInternal();
}

void FDynamicResolutionHeuristicProxy::ResetInternal()
{
	PreviousFrameIndex = -1;
	HistorySize = 0;
	BudgetHistorySizes.SetAll(0);
	History.Reset();

	NumberOfFramesSinceScreenPercentageChange = 0;
	CurrentFrameResolutionFractions.SetAll(1.0f);
	CurrentFrameMaxResolutionFractions.SetAll(1.0f);

	TemporalUpscalerMinResolutionFraction = ISceneViewFamilyScreenPercentage::kMinResolutionFraction;
	TemporalUpscalerMaxResolutionFraction = ISceneViewFamilyScreenPercentage::kMaxResolutionFraction;

	// Ignore previous frame timings.
	IgnoreFrameRemainingCount = 1;
}

void FDynamicResolutionHeuristicProxy::CreateNewPreviousFrameTimings_RenderThread(float GameThreadTimeMs, float RenderThreadTimeMs, float TotalFrameGPUBusyTimeMs)
{
	check(IsInRenderingThread());
	check(TotalFrameGPUBusyTimeMs >= 0.0f);

	// Early return if want to ignore frames.
	if (IgnoreFrameRemainingCount > 0)
	{
		IgnoreFrameRemainingCount--;
		return;
	}

	ResizeHistoryIfNeeded();

	// Update history state.
	int32 NewHistoryEntryIndex = (PreviousFrameIndex + 1) % History.Num();
	FrameHistoryEntry& Entry = History[NewHistoryEntryIndex];
	PreviousFrameIndex = NewHistoryEntryIndex;

	Entry = FrameHistoryEntry();
	Entry.ResolutionFractions = CurrentFrameResolutionFractions;
	Entry.GameThreadTimeMs = GameThreadTimeMs;
	Entry.RenderThreadTimeMs = RenderThreadTimeMs;
	Entry.TotalFrameGPUBusyTimeMs = TotalFrameGPUBusyTimeMs;
		
	HistorySize = FMath::Min(HistorySize + 1, History.Num());

	const DynamicRenderScaling::TMap<uint64>& LatestTimings = DynamicRenderScaling::GetLatestTimings();

	Entry.BudgetTimingMs[GDynamicPrimaryResolutionFraction] = TotalFrameGPUBusyTimeMs;

	for (const DynamicRenderScaling::FBudget* Budget : *DynamicRenderScaling::FBudget::GetGlobalList())
	{
		Entry.BudgetTimingMs[*Budget] = TimeStampQueryResultToMilliSeconds(LatestTimings[*Budget]);
		BudgetHistorySizes[*Budget] = FMath::Min(BudgetHistorySizes[*Budget] + 1, History.Num());
	}
}


void FDynamicResolutionHeuristicProxy::RefreshCurrentFrameResolutionFraction_RenderThread()
{
	// Global constants.
	const float FrameWeightExponent = CVarFrameWeightExponent.GetValueOnRenderThread();
	const int32 MaxConsecutiveOverBudgetGPUFrameCount = FMath::Max(CVarMaxConsecutiveOverBudgetGPUFrameCount.GetValueOnRenderThread(), 2);

	const bool bCanChangeResolution = NumberOfFramesSinceScreenPercentageChange >= CVarFrameChangePeriod.GetValueOnRenderThread();
	const bool bUseCPUTimeLogic = CVarUseCPUTimeLogic.GetValueOnRenderThread();

	// New ResolutionFraction to use for this frame.
	DynamicRenderScaling::TMap<float> NewFrameResolutionFractions = CurrentFrameResolutionFractions;

	// Whether there is a GPU over budget panic.
	bool bGlobalGPUOverBudgetPanic = false;

	for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
	{
		const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
		const DynamicRenderScaling::FHeuristicSettings& BudgetSettings = Budget.GetSettings();

		const int32 BudgetHistorySize = FMath::Min(HistorySize, BudgetHistorySizes[Budget]);

		// Early returns if not enough data to work with.
		if (BudgetHistorySize == 0 || (Budget != GDynamicPrimaryResolutionFraction && !BudgetSettings.IsEnabled()))
		{
			continue;
		}

		float BudgetBudgetMs = BudgetSettings.BudgetMs;
		const float BudgetTargetMs = BudgetSettings.GetTargetedMs(BudgetBudgetMs);

		float NewFrameResolutionFraction = 0.0f;

		// Total weight of NewFrameResolutionFraction.
		float TotalWeight = 0.0f;

		// Frame weight.
		float Weight = 1.0f;

		// Number of consecutive frames that have over budget GPU.
		int32 ConsecutiveOverBudgetGPUFrameCount = 0;

		// Whether should continue browsing the frame history.
		bool bCarryOnBrowsingFrameHistory = true;

		// Number of frame browsed.
		int32 FrameCount = 0;
		for (int32 BrowsingFrameId = 0; BrowsingFrameId < BudgetHistorySize && bCarryOnBrowsingFrameHistory; BrowsingFrameId++)
		{
			const FrameHistoryEntry& FrameEntry = GetPreviousFrameEntry(BrowsingFrameId);

			const float TotalFrameGPUBusyTimeMs = Budget == GDynamicPrimaryResolutionFraction
				? FrameEntry.TotalFrameGPUBusyTimeMs
				: FrameEntry.BudgetTimingMs[Budget];

			// Ignores frames that does not have any GPU timing yet. 
			if (TotalFrameGPUBusyTimeMs < 0)
			{
				continue;
			}

		#if STATS
			SET_FLOAT_STAT_FName(Budget.GetStatId_MeasuredMs().GetName(), TotalFrameGPUBusyTimeMs);
		#endif

			// Whether bound by game thread.
			const bool bIsGameThreadBound = FrameEntry.GameThreadTimeMs > GDynamicPrimaryResolutionFraction.GetSettings().BudgetMs;

			// Whether bound by render thread.
			const bool bIsRenderThreadBound = FrameEntry.RenderThreadTimeMs > GDynamicPrimaryResolutionFraction.GetSettings().GetTargetedMs(GDynamicPrimaryResolutionFraction.GetSettings().BudgetMs);

			// Whether the frame is CPU bound.
			const bool bIsCPUBound = bUseCPUTimeLogic && (bIsGameThreadBound || bIsRenderThreadBound);

			// Whether GPU is over budget, when not CPU bound.
			const bool bHasOverBudgetGPU = !bIsCPUBound && TotalFrameGPUBusyTimeMs > BudgetBudgetMs;

			// Look if this is multiple consecutive GPU over budget frames.
			if (bHasOverBudgetGPU)
			{
				ConsecutiveOverBudgetGPUFrameCount++;
				check(ConsecutiveOverBudgetGPUFrameCount <= MaxConsecutiveOverBudgetGPUFrameCount);

				// Max number of over budget frames where reached.
				if (ConsecutiveOverBudgetGPUFrameCount == MaxConsecutiveOverBudgetGPUFrameCount)
				{
					// We ignore frames in history that happen before consecutive GPU over budget frames.
					bCarryOnBrowsingFrameHistory = false;
				}
			}
			else
			{
				ConsecutiveOverBudgetGPUFrameCount = 0;
			}

			float SuggestedResolutionFraction = 1.0f;

			// If we have reliable GPU times, or guess there are no GPU bubbles -> estimate the suggested resolution fraction that could have been used.
			{
				// This assumes GPU busy time is directly proportional to ResolutionFraction^2, but in practice
				// this is more A * ResolutionFraction^2 + B with B >= 0 non constant unknown cost such as unscaled
				// post processing, vertex fetching & processing, occlusion queries, shadow map rendering...
				//
				// This assumption means we may drop ResolutionFraction lower than needed, or be slower to increase
				// resolution.
				//
				// TODO: If we have RHI guarantee of frame timing association, we could make an estimation of B
				// At resolution change that happen every N frames, amortized over time and scaled down as the
				// standard variation of the GPU timing over non resolution changing frames increases.
				SuggestedResolutionFraction = BudgetSettings.EstimateResolutionFactor(BudgetTargetMs, TotalFrameGPUBusyTimeMs) * FrameEntry.ResolutionFractions[Budget];
			}

			NewFrameResolutionFraction += SuggestedResolutionFraction * Weight;
			TotalWeight += Weight;
			FrameCount++;

			Weight *= FrameWeightExponent;
		}

		NewFrameResolutionFraction /= TotalWeight;

		// If immediate previous frames where over budget, react immediately.
		bool bGPUOverBudgetPanic = FrameCount > 0 && ConsecutiveOverBudgetGPUFrameCount == FrameCount;

		// If over budget, reset history size to 0 so that this frame really behave as a first frame after an history reset.
		if (bGPUOverBudgetPanic)
		{
			BudgetHistorySizes[Budget] = 0;

			if (Budget == GDynamicPrimaryResolutionFraction)
			{
				HistorySize = 0;
				bGlobalGPUOverBudgetPanic = true;
			}
		}
		// If not immediately over budget, refine the new resolution fraction.
		else
		{
			// If scaling the resolution, look if this is above a threshold compared to current res.
			if (!BudgetSettings.DoesResolutionChangeEnough(CurrentFrameResolutionFractions[Budget], NewFrameResolutionFraction, bCanChangeResolution))
			{
				NewFrameResolutionFraction = CurrentFrameResolutionFractions[Budget];
			}

			// If scaling the resolution up, amortize to avoid oscillations.
			if (NewFrameResolutionFraction > CurrentFrameResolutionFractions[Budget])
			{
				NewFrameResolutionFraction = FMath::Lerp(
					CurrentFrameResolutionFractions[Budget],
					NewFrameResolutionFraction,
					BudgetSettings.IncreaseAmortizationFactor);
			}
		}

		float FinalMaxResolutionFraction = BudgetSettings.MaxResolutionFraction;
		if (BudgetSettings.ThrottlingMaxResolutionFraction > 0.0f)
		{
			// Don't allow the throttling to resolution to mess up with the primary MinResolutionFraction and MaxResolutionFraction settings.
			FinalMaxResolutionFraction = FMath::Clamp(
				BudgetSettings.ThrottlingMaxResolutionFraction,
				BudgetSettings.MinResolutionFraction,
				BudgetSettings.MaxResolutionFraction);
		}

		// Clamp resolution fraction.
		NewFrameResolutionFraction = FMath::Clamp(
			NewFrameResolutionFraction,
			BudgetSettings.MinResolutionFraction,
			FinalMaxResolutionFraction);

		// Also clamp with the temporal upscaler's minimum and maximum fractions (set to theoretical minimum and maximum if not in use)
		NewFrameResolutionFraction = FMath::Clamp(
			NewFrameResolutionFraction,
			TemporalUpscalerMinResolutionFraction,
			TemporalUpscalerMaxResolutionFraction);

		NewFrameResolutionFractions[Budget] = NewFrameResolutionFraction;
	}

	// Update the current frame's resolution fraction.
	{
		bool bWouldBeWorthChangingRes = false;
		for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
		{
			const DynamicRenderScaling::FBudget& Budget = **BudgetIt;

			// CVarChangeThreshold avoids very small changes.
			bWouldBeWorthChangingRes = bWouldBeWorthChangingRes || CurrentFrameResolutionFractions[Budget] != NewFrameResolutionFractions[Budget];
		}
		
		// We do not change resolution too often to avoid interferences with temporal sub pixel in TAA upsample.
		if ((bWouldBeWorthChangingRes && bCanChangeResolution) || bGlobalGPUOverBudgetPanic)
		{
			NumberOfFramesSinceScreenPercentageChange = 0;
			CurrentFrameResolutionFractions = NewFrameResolutionFractions;
		}
		else
		{
			NumberOfFramesSinceScreenPercentageChange++;
		}
	}
	
	RefreshCurrentFrameResolutionFractionUpperBound_RenderThread();
	RefreshHeuristicStats_RenderThread();
}

void FDynamicResolutionHeuristicProxy::SetTemporalUpscaler(const UE::Renderer::Private::ITemporalUpscaler* InTemporalUpscaler)
{
	check(IsInParallelRenderingThread());

	float NewMinResolutionFraction = InTemporalUpscaler ? InTemporalUpscaler->GetMinUpsampleResolutionFraction() : ISceneViewFamilyScreenPercentage::kMinResolutionFraction;
	float NewMaxResolutionFraction = InTemporalUpscaler ? InTemporalUpscaler->GetMaxUpsampleResolutionFraction() : ISceneViewFamilyScreenPercentage::kMaxResolutionFraction;

	if (NewMinResolutionFraction != TemporalUpscalerMinResolutionFraction ||
		NewMaxResolutionFraction != TemporalUpscalerMaxResolutionFraction)
	{
		TemporalUpscalerMinResolutionFraction = NewMinResolutionFraction;
		TemporalUpscalerMaxResolutionFraction = NewMaxResolutionFraction;

		// If the temporal upscaler (or its supported range) have changed, refresh the fractions for this frame
		RefreshCurrentFrameResolutionFraction_RenderThread();
	}
}

void FDynamicResolutionHeuristicProxy::RefreshCurrentFrameResolutionFractionUpperBound_RenderThread()
{
	// Compute max resolution for each budget by quantizing the new resolution fraction (falls back to the MaxResolution setting if BudgetSetting.UpperBoundQuantization==0)
	DynamicRenderScaling::TMap<float> NewMaxResolutionFractions;
	for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
	{
		const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
		DynamicRenderScaling::FHeuristicSettings BudgetSettings = Budget.GetSettings();

		if (BudgetSettings.IsEnabled() || Budget == GDynamicPrimaryResolutionFraction)
		{
			float NewMaxResolutionFraction = BudgetSettings.MaxResolutionFraction;
			if (BudgetSettings.UpperBoundQuantization > 0)
			{
				float CurrentResolutionFraction = CurrentFrameResolutionFractions[Budget];
				float AvailableRange = BudgetSettings.MaxResolutionFraction - BudgetSettings.MinResolutionFraction;
				float QuantizationStepSize = AvailableRange / float(BudgetSettings.UpperBoundQuantization);
				NewMaxResolutionFraction = FMath::CeilToFloat(CurrentResolutionFraction / QuantizationStepSize) * QuantizationStepSize;
				NewMaxResolutionFraction = FMath::Min(NewMaxResolutionFraction, BudgetSettings.MaxResolutionFraction);
			}
			NewMaxResolutionFractions[Budget] = NewMaxResolutionFraction;
		}
		else
		{
			NewMaxResolutionFractions[Budget] = 1.0f;
		}
	}

	CurrentFrameMaxResolutionFractions = NewMaxResolutionFractions;
}

void FDynamicResolutionHeuristicProxy::RefreshHeuristicStats_RenderThread()
{
#if STATS
	check(IsInParallelRenderingThread());
	for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
	{
		const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
		const DynamicRenderScaling::FHeuristicSettings& HeuristicSettings = Budget.GetSettings();

		if (HeuristicSettings.IsEnabled())
		{
			SET_FLOAT_STAT_FName(Budget.GetStatId_TargetMs().GetName(), HeuristicSettings.GetTargetedMs(HeuristicSettings.BudgetMs));
			// MeasuredMs is set in RefreshCurrentFrameResolutionFraction_RenderThread()
			SET_FLOAT_STAT_FName(Budget.GetStatId_MinScaling().GetName(), HeuristicSettings.MinResolutionFraction);
			SET_FLOAT_STAT_FName(Budget.GetStatId_MaxScaling().GetName(), HeuristicSettings.MaxResolutionFraction);
			SET_FLOAT_STAT_FName(Budget.GetStatId_CurrentScaling().GetName(), CurrentFrameResolutionFractions[Budget]);
		}
		else
		{
			SET_FLOAT_STAT_FName(Budget.GetStatId_TargetMs().GetName(), 0.0f);
			SET_FLOAT_STAT_FName(Budget.GetStatId_MeasuredMs().GetName(), 0.0f);
			SET_FLOAT_STAT_FName(Budget.GetStatId_MinScaling().GetName(), 0.0f);
			SET_FLOAT_STAT_FName(Budget.GetStatId_MaxScaling().GetName(), 0.0f);
			SET_FLOAT_STAT_FName(Budget.GetStatId_CurrentScaling().GetName(), 0.0f);
		}
	}
#endif
}

// static
DynamicRenderScaling::TMap<float> FDynamicResolutionHeuristicProxy::GetResolutionFractionUpperBounds() const
{
	check(IsInGameThread() || IsInParallelRenderingThread());
	return CurrentFrameMaxResolutionFractions;
}


/** Returns the view fraction that should be used for current frame. */
DynamicRenderScaling::TMap<float> FDynamicResolutionHeuristicProxy::QueryCurrentFrameResolutionFractions_Internal() const
{
	DynamicRenderScaling::TMap<float> MaxResolutionFractions = GetResolutionFractionUpperBounds();
	DynamicRenderScaling::TMap<float> ResolutionFractions = CurrentFrameResolutionFractions;
	for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
	{
		const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
		ResolutionFractions[Budget] = FMath::Min(ResolutionFractions[Budget], MaxResolutionFractions[Budget]);
	}
	return ResolutionFractions;
}

void FDynamicResolutionHeuristicProxy::ResizeHistoryIfNeeded()
{
	uint32 DesiredHistorySize = FMath::Max(1, CVarHistorySize.GetValueOnRenderThread());

	if (History.Num() == DesiredHistorySize)
	{
		return;
	}

	TArray<FrameHistoryEntry> NewHistory;
	NewHistory.SetNum(DesiredHistorySize);

	int32 NewHistorySize = FMath::Min(HistorySize, NewHistory.Num());
	int32 NewPreviousFrameIndex = NewHistorySize - 1;

	for (int32 i = 0; i < NewHistorySize; i++)
	{
		NewHistory[NewPreviousFrameIndex - i] = History[(PreviousFrameIndex - i) % History.Num()];
	}

	History = NewHistory;
	HistorySize = NewHistorySize;
	PreviousFrameIndex = NewPreviousFrameIndex;
}


/**
 * Render thread proxy for engine's dynamic resolution state.
 */
class FDefaultDynamicResolutionStateProxy
{
public:
	FDefaultDynamicResolutionStateProxy()
	{
		check(IsInGameThread());
	}

	~FDefaultDynamicResolutionStateProxy()
	{
		check(IsInRenderingThread());
	}

	void Reset()
	{
		check(IsInRenderingThread());

		// Reset heuristic.
		Heuristic.Reset_RenderThread();
	}

	void BeginFrame(float PrevGameThreadTimeMs)
	{
		check(IsInRenderingThread());
		ensure(GRHISupportsFrameCyclesBubblesRemoval);

		if (DynamicRenderScaling::IsSupported())
		{
			DynamicRenderScaling::UpdateHeuristicsSettings();

			DynamicRenderScaling::TMap<bool> bIsBudgetEnabled;
			bIsBudgetEnabled.SetAll(false);

			for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
			{
				const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
				bIsBudgetEnabled[Budget] = Budget.GetSettings().IsEnabled();
			}

			DynamicRenderScaling::BeginFrame(bIsBudgetEnabled);
		}

		float PrevRenderThreadTimeMs = FPlatformTime::ToMilliseconds(GRenderThreadTime);

		uint64 GPUFrameTimeCycles64;
		while (GPUFrameTimeState.PopFrameCycles(GPUFrameTimeCycles64) != FRHIGPUFrameTimeHistory::EResult::Empty)
		{
			float PrevFrameGPUTimeMs = FPlatformTime::ToMilliseconds64(GPUFrameTimeCycles64);
			Heuristic.CreateNewPreviousFrameTimings_RenderThread(PrevGameThreadTimeMs, PrevRenderThreadTimeMs, PrevFrameGPUTimeMs);
			Heuristic.RefreshCurrentFrameResolutionFraction_RenderThread();
		}
	}

	void ProcessEvent(EDynamicResolutionStateEvent Event)
	{
		check(IsInRenderingThread());

		if (Event == EDynamicResolutionStateEvent::EndFrame)
		{
			DynamicRenderScaling::EndFrame();
		}
	}

	/// Called before object is to be deleted
	void Finalize()
	{
		check(IsInRenderingThread());
	}

	// Heuristic's proxy.
	FDynamicResolutionHeuristicProxy Heuristic;

	FRHIGPUFrameTimeHistory::FState GPUFrameTimeState;
};


/**
 * Engine's default dynamic resolution driver for view families.
 */
class FDefaultDynamicResolutionDriver : public ISceneViewFamilyScreenPercentage
{
public:

	FDefaultDynamicResolutionDriver(FDefaultDynamicResolutionStateProxy* InProxy, const FSceneViewFamily& InViewFamily)
		: Proxy(InProxy)
		, ViewFamily(InViewFamily)
	{
		check(IsInGameThread());
	}

	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsUpperBound() const override
	{
		DynamicRenderScaling::TMap<float> UpperBounds = Proxy->Heuristic.GetResolutionFractionUpperBounds();
		if (!ViewFamily.EngineShowFlags.ScreenPercentage)
		{
			UpperBounds[GDynamicPrimaryResolutionFraction] = 1.0f;
		}

		return UpperBounds;
	}

	virtual ISceneViewFamilyScreenPercentage* Fork_GameThread(const class FSceneViewFamily& ForkedViewFamily) const override
	{
		check(IsInGameThread());

		return new FDefaultDynamicResolutionDriver(Proxy, ForkedViewFamily);
	}

	virtual DynamicRenderScaling::TMap<float> GetResolutionFractions_RenderThread() const override
	{
		check(IsInParallelRenderingThread());

		DynamicRenderScaling::TMap<float> ResolutionFractions = Proxy->Heuristic.QueryCurrentFrameResolutionFractions();
		if (!ViewFamily.EngineShowFlags.ScreenPercentage)
		{
			ResolutionFractions[GDynamicPrimaryResolutionFraction] = 1.0f;
		}

		return ResolutionFractions;
	}

private:
	// Dynamic resolution proxy to use.
	FDefaultDynamicResolutionStateProxy* Proxy;

	// View family to take care of.
	const FSceneViewFamily& ViewFamily;

};


/**
 * Engine's default dynamic resolution state.
 */
class FDefaultDynamicResolutionState : public IDynamicResolutionState
{
public:
	FDefaultDynamicResolutionState()
		: Proxy(new FDefaultDynamicResolutionStateProxy())
	{
		check(IsInGameThread());
		bIsEnabled = false;
		bRecordThisFrame = false;
	}

	~FDefaultDynamicResolutionState() override
	{
		check(IsInGameThread());

		// Deletes the proxy on the rendering thread to make sure we don't delete before a recommand using it has finished.
		ENQUEUE_RENDER_COMMAND(DeleteDynamicResolutionProxy)(
			[P = Proxy](class FRHICommandList&)
		{
			P->Finalize();
			delete P;
		});
	}


	// Implements IDynamicResolutionState

	virtual bool IsSupported() const override
	{
		// No VR platforms are officially supporting dynamic resolution with Engine default's dynamic resolution state.
		const bool bIsStereo = GEngine->StereoRenderingDevice.IsValid() ? GEngine->StereoRenderingDevice->IsStereoEnabled() : false;
		if (bIsStereo)
		{
			return false;
		}
		return GRHISupportsDynamicResolution;
	}

	virtual void ResetHistory() override
	{
		check(IsInGameThread());

		ENQUEUE_RENDER_COMMAND(DynamicResolutionResetHistory)(
			[P = Proxy](class FRHICommandList&)
		{
			P->Reset();
		});

	}

	virtual void SetEnabled(bool bEnable) override
	{
		check(IsInGameThread());
		bIsEnabled = bEnable;
	}

	virtual bool IsEnabled() const override
	{
		check(IsInGameThread());
		return bIsEnabled;
	}

	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsApproximation() const override
	{
		check(IsInGameThread());
		return Proxy->Heuristic.GetResolutionFractionsApproximation_GameThread();
	}

	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsUpperBound() const override
	{
		check(IsInGameThread());
		return Proxy->Heuristic.GetResolutionFractionUpperBounds();
	}

	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsUpperBoundBudgetValue() const override
	{
		DynamicRenderScaling::TMap<float> MaxResolutionFractions;
		for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
		{
			const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
			MaxResolutionFractions[Budget] = Budget.GetSettings().MaxResolutionFraction;
		}
		return MaxResolutionFractions;
	}

	virtual void ProcessEvent(EDynamicResolutionStateEvent Event) override
	{
		check(IsInGameThread());

		if (Event == EDynamicResolutionStateEvent::BeginFrame)
		{
			check(bRecordThisFrame == false);
			bRecordThisFrame = bIsEnabled && IsSupported();
		}

		// Early return if not recording this frame.
		if (!bRecordThisFrame)
		{
			return;
		}

		if (Event == EDynamicResolutionStateEvent::BeginFrame)
		{
			float PrevGameThreadTimeMs = FPlatformTime::ToMilliseconds(GGameThreadTime);

			if (CVarUseGameThreadCriticalPath.GetValueOnAnyThread())
			{
				PrevGameThreadTimeMs = FPlatformTime::ToMilliseconds(GGameThreadTimeCriticalPath);
			}

			ENQUEUE_RENDER_COMMAND(DynamicResolutionBeginFrame)(
				[P = Proxy, PrevGameThreadTimeMs](FRHICommandList&)
			{
				P->BeginFrame(PrevGameThreadTimeMs);
			});
		}
		else
		{
			// Forward event to render thread.
			ENQUEUE_RENDER_COMMAND(DynamicResolutionBeginFrame)(
				[P = Proxy, Event](FRHICommandList&)
			{
				P->ProcessEvent(Event);
			});

			if (Event == EDynamicResolutionStateEvent::EndFrame)
			{
				// Only record frames that have a BeginFrame event.
				bRecordThisFrame = false;
			}
		}
	}

	virtual void SetupMainViewFamily(class FSceneViewFamily& ViewFamily) override
	{
		check(IsInGameThread());

		if (bIsEnabled)
		{
			ViewFamily.SetScreenPercentageInterface(new FDefaultDynamicResolutionDriver(Proxy, ViewFamily));
		}
	}

	virtual void SetTemporalUpscaler(const UE::Renderer::Private::ITemporalUpscaler* InTemporalUpscaler)
	{
		Proxy->Heuristic.SetTemporalUpscaler(InTemporalUpscaler);
	}

private:
	// Owned render thread proxy.
	FDefaultDynamicResolutionStateProxy* const Proxy;

	// Whether dynamic resolution is enabled.
	bool bIsEnabled;

	// Whether dynamic resolution is recording this frame.
	bool bRecordThisFrame;
};


//static
TSharedPtr< class IDynamicResolutionState > FDynamicResolutionHeuristicProxy::CreateDefaultState()
{
	return MakeShareable(new FDefaultDynamicResolutionState());
}
