// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"
#include "GraphEditAction.h"
#include "EdGraph/EdGraphSchema.h"
#include "EngineLogs.h"
#include "UObject/LinkerLoad.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraph)

#if WITH_EDITOR
#endif

#define LOCTEXT_NAMESPACE "EdGraph"

DEFINE_LOG_CATEGORY_STATIC(LogEdGraph, Log, All);

/////////////////////////////////////////////////////
// FGraphReference

void FGraphReference::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.UEVer() >= VER_UE4_K2NODE_REFERENCEGUIDS)
	{
		// Because the macro instance could have been saved with a GUID that was allocated 
		// but the macro graph never actually saved with that value we are forced to make 
		// sure to refresh the GUID and make sure it is up to date
		if (MacroGraph)
		{
			GraphGuid = MacroGraph->GraphGuid;
		}
	}
#endif
}

#if WITH_EDITORONLY_DATA
void FGraphReference::SetGraph(UEdGraph* InGraph)
{
	MacroGraph = InGraph;
	if (InGraph)
	{
		GraphBlueprint = InGraph->GetTypedOuter<UBlueprint>();
		GraphGuid = InGraph->GraphGuid;
	}
	else
	{
		GraphBlueprint = NULL;
		GraphGuid.Invalidate();
	}
}

UEdGraph* FGraphReference::GetGraph() const
{
	if (MacroGraph == NULL)
	{
		if (GraphBlueprint)
		{
			TArray<UObject*> ObjectsInPackage;
			GetObjectsWithOuter(GraphBlueprint, ObjectsInPackage);

			for (int32 Index = 0; Index < ObjectsInPackage.Num(); ++Index)
			{
				UEdGraph* FoundGraph = Cast<UEdGraph>(ObjectsInPackage[Index]);
				if (FoundGraph && FoundGraph->GraphGuid == GraphGuid)
				{
					MacroGraph = FoundGraph;
					break;
				}
			}
		}
	}

	return MacroGraph;
}
#endif

/////////////////////////////////////////////////////
// UEdGraph

UEdGraph::UEdGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bEditable = true;
	bAllowDeletion = true;
}

#if WITH_EDITORONLY_DATA
void UEdGraph::BuildSubobjectMapping(UObject* OtherObject, TMap<UObject*, UObject*>& ObjectMapping) const
{
	UEdGraph* OtherGraph = CastChecked<UEdGraph>(OtherObject);

	auto FindMatchingNode = [](UEdGraph* InGraphToSearch, UEdGraphNode* InNodeToFind) -> UEdGraphNode*
	{
		TArray<UEdGraphNode*, TInlineAllocator<8>> PotentialMatches;

		for (UEdGraphNode* GraphNode : InGraphToSearch->Nodes)
		{
			if (GraphNode->GetClass() == InNodeToFind->GetClass())
			{
				// Ideally the node would match by GUID or name
				if (GraphNode->NodeGuid == InNodeToFind->NodeGuid || GraphNode->GetFName() == InNodeToFind->GetFName())
				{
					return GraphNode;
				}

				// If it doesn't match by either, we may be searching an older graph without stable node IDs... consider the node title as significant in this case
				{
					const FText Title1 = GraphNode->GetNodeTitle(ENodeTitleType::FullTitle);
					const FText Title2 = InNodeToFind->GetNodeTitle(ENodeTitleType::FullTitle);

					if (Title1.ToString().Equals(Title2.ToString(), ESearchCase::CaseSensitive))
					{
						PotentialMatches.Emplace(GraphNode);
					}
				}
			}
		}

		// Only use the title-based resolution if we found a single match!
		if (PotentialMatches.Num() == 1)
		{
			return PotentialMatches[0];
		}

		return nullptr;
	};

	for (UEdGraphNode* GraphNode : Nodes)
	{
		if (GraphNode && !ObjectMapping.Contains(GraphNode))
		{
			UEdGraphNode* OtherGraphNode = FindMatchingNode(OtherGraph, GraphNode);
			ObjectMapping.Emplace(GraphNode, OtherGraphNode);
		}
	}

	Super::BuildSubobjectMapping(OtherObject, ObjectMapping);
}

void UEdGraph::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		GraphGuid = FGuid::NewGuid();
	}
}

void UEdGraph::Serialize(FStructuredArchiveRecord Record)
{
	Super::Serialize(Record);
	const FArchiveState& ArchiveState = Record.GetArchiveState();

	// Keep track of RF_Public
	if(ArchiveState.IsTransacting())
	{
		bool bIsPublic = HasAnyFlags(RF_Public);
		if(ArchiveState.IsLoading())
		{
			Record << SA_VALUE(TEXT("IsPublic"), bIsPublic);
			if (bIsPublic)
			{
				SetFlags( RF_Public );
			}
			else
			{
				ClearFlags( RF_Public );
			}
		}
		else if( ArchiveState.IsSaving())
		{
			Record << SA_VALUE(TEXT("IsPublic"), bIsPublic);
		}
	}
}

void UEdGraph::PostLoad()
{
	Super::PostLoad();

	// Strip out null nodes (likely from missing node classes) as they will cause crashes 
	for (int32 i = Nodes.Num() - 1; i >= 0; i--)
	{
		if (Nodes[i] == nullptr)
		{
			Nodes.RemoveAt(i);
			UE_LOGF(LogBlueprint, Display, 
				"Missing node found in the EdGraph. We've fixed up the issue, but couldn't determine the cause. If you are able to figure out the cause, please notify the engine team; otherwise consider resaving the asset: '%ls'",
				*GetOutermost()->GetName());
		}
	}
}
#endif

const UEdGraphSchema* UEdGraph::GetSchema() const
{
	if (Schema == NULL)
	{
		return NULL;
	}
	return GetDefault<UEdGraphSchema>(Schema);
}

FDelegateHandle UEdGraph::AddOnGraphChangedHandler( const FOnGraphChanged::FDelegate& InHandler )
{
	return OnGraphChanged.Add( InHandler );
}

void UEdGraph::RemoveOnGraphChangedHandler( FDelegateHandle Handle )
{
	OnGraphChanged.Remove( Handle );
}

UEdGraphNode* UEdGraph::CreateNode( TSubclassOf<UEdGraphNode> NewNodeClass, bool bFromUI, bool bSelectNewNode )
{
	UEdGraphNode* NewNode = NewObject<UEdGraphNode>(this, NewNodeClass, NAME_None, RF_Transactional);

	if (HasAnyFlags(RF_Transient))
	{
		NewNode->SetFlags(RF_Transient);
	}

	AddNode(NewNode, bFromUI, bSelectNewNode );
	return NewNode;
}

void UEdGraph::AddNode( UEdGraphNode* NodeToAdd, bool bFromUI/* = false*/, bool bSelectNewNode/* = true*/ )
{
	this->Nodes.Add(NodeToAdd);
	check(NodeToAdd->GetOuter() == this);

	// Create the graph
	EEdGraphActionType AddNodeAction = GRAPHACTION_AddNode;

	if (bSelectNewNode)
	{
		AddNodeAction = (EEdGraphActionType)( ((int32)AddNodeAction) | GRAPHACTION_SelectNode );
	}

	FEdGraphEditAction Action(AddNodeAction, this, NodeToAdd, bFromUI);
	
	NotifyGraphChanged( Action );
}

void UEdGraph::SelectNodeSet(TSet<const UEdGraphNode*> NodeSelection, bool bFromUI/*= false*/) 
{
	FEdGraphEditAction SelectionAction;

	SelectionAction.Action = GRAPHACTION_SelectNode;

	SelectionAction.Graph = this;
	SelectionAction.Nodes = NodeSelection;
	SelectionAction.bUserInvoked = bFromUI;

	NotifyGraphChanged(SelectionAction);
}

bool UEdGraph::RemoveNode( UEdGraphNode* NodeToRemove, bool bBreakAllLinks, bool bAlwaysMarkDirty )
{
	Modify(bAlwaysMarkDirty);
	
	FLinkerLoad::InvalidateExport(NodeToRemove);
	int32 NumTimesNodeRemoved = Nodes.Remove(NodeToRemove);
#if WITH_EDITOR
	if (bBreakAllLinks)
	{
		NodeToRemove->BreakAllNodeLinks(bAlwaysMarkDirty);
	}
#endif	//#if WITH_EDITOR

	FEdGraphEditAction RemovalAction;
	RemovalAction.Graph = this;
	RemovalAction.Action = GRAPHACTION_RemoveNode;
	RemovalAction.Nodes.Add(NodeToRemove);
	NotifyGraphChanged(RemovalAction);

	return NumTimesNodeRemoved > 0;
}

void UEdGraph::NotifyGraphChanged()
{
	FEdGraphEditAction Action;
	OnGraphChanged.Broadcast(Action);
}

void UEdGraph::NotifyNodeChanged(const UEdGraphNode* Node)
{
	FEdGraphEditAction Action;
	Action.Graph = this;
	Action.Action = GRAPHACTION_EditNode;
	Action.Nodes.Add(Node);
	NotifyGraphChanged(Action);
}

void UEdGraph::NotifyGraphChanged(const FEdGraphEditAction& InAction)
{
	OnGraphChanged.Broadcast(InAction);
}

void UEdGraph::MoveNodesToAnotherGraph(UEdGraph* DestinationGraph, bool bIsLoading, bool bInIsCompiling)
{
	// Move one node over at a time
	DestinationGraph->Nodes.Reserve(DestinationGraph->Nodes.Num() + Nodes.Num());
	while (Nodes.Num())
	{
		if (UEdGraphNode* Node = Nodes.Pop(EAllowShrinking::No))
		{
			// Let the name be autogenerated, to automatically avoid naming conflicts
			// Since this graph is always going to come from a cloned source graph, user readable names can come from the remap stored in a MessageLog.
			Node->Rename(/*NewName=*/ nullptr, /*NewOuter=*/ DestinationGraph, REN_DontCreateRedirectors | (bInIsCompiling ? REN_NonTransactional : 0) | (bIsLoading ? REN_AllowPackageLinkerMismatch : 0));
	
			DestinationGraph->Nodes.Add(Node);
		}
	}

	DestinationGraph->NotifyGraphChanged();
	NotifyGraphChanged();
}

void UEdGraph::GetAllChildrenGraphs(TArray<UEdGraph*>& Graphs) const
{
#if WITH_EDITORONLY_DATA
	for (int32 i = 0; i < SubGraphs.Num(); ++i)
	{
		UEdGraph* Graph = SubGraphs[i];
		if (ensureMsgf(Graph, TEXT("%s has invalid SubGraph array entry at %d"), *GetFullName(), i))
		{
			Graphs.Add(Graph);
			Graph->GetAllChildrenGraphs(Graphs);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

UEdGraph* UEdGraph::GetOuterGraph(UObject* Obj)
{
	if (Obj)
	{
		UObject* OuterObj = Obj->GetOuter();
		if (OuterObj)
		{
			if (OuterObj->IsA(UEdGraph::StaticClass()))
			{
				return Cast<UEdGraph>(OuterObj);
			}
			else
			{
				return GetOuterGraph(OuterObj);
			}
		}
	}
	return nullptr;
}

UE::Slate::FDeprecateVector2DResult UEdGraph::GetGoodPlaceForNewNode()
{
	FVector2f BottomLeft(0.0f, 0.0f);

	if(Nodes.Num() > 0)
	{
		UEdGraphNode* Node = Nodes[0];
		if (Node)
		{
			BottomLeft = FVector2f(Node->NodePosX, Node->NodePosY);
			for (int32 i = 1; i < Nodes.Num(); i++)
			{
				Node = Nodes[i];
				if (Node)
				{
					BottomLeft.X = FMath::Min(BottomLeft.X, Node->NodePosX);
					BottomLeft.Y = FMath::Max(BottomLeft.Y, Node->NodePosY);
				}
			}
		}
	}

	return BottomLeft + FVector2f(0.0f, 256.0f);
}

#if WITH_EDITOR

void UEdGraph::NotifyPreChange( const FString& PropertyName )
{
	// no notification is hooked up yet
}

void UEdGraph::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, const FString& PropertyName )
{
#if WITH_EDITORONLY_DATA
	PropertyChangedNotifiers.Broadcast(PropertyChangedEvent, PropertyName);
#endif
}

FDelegateHandle UEdGraph::AddPropertyChangedNotifier(const FOnPropertyChanged::FDelegate& InDelegate )
{
#if WITH_EDITORONLY_DATA
	return PropertyChangedNotifiers.Add(InDelegate);
#else
	return FDelegateHandle();
#endif
}

void UEdGraph::RemovePropertyChangedNotifier(FDelegateHandle Handle )
{
#if WITH_EDITORONLY_DATA
	PropertyChangedNotifiers.Remove(Handle);
#endif
}
#endif	//WITH_EDITOR

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

