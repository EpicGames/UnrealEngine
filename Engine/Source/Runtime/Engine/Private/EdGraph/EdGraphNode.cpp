// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/EdGraphNode.h"

#include "Serialization/PropertyLocalizationDataGathering.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Textures/SlateIcon.h"
#include "EngineLogs.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphNode)
#if WITH_EDITOR
#include "CookerSettings.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/FeedbackContext.h"
#include "ScopedTransaction.h"
#include "FindInBlueprintManager.h"
#include "DiffResults.h"
#include "DiffUtils.h"
#else
#include "EdGraph/EdGraphPin.h"
#endif

#define LOCTEXT_NAMESPACE "EdGraph"

static int32 GEdGraphStripNodeComment = 0;
static FAutoConsoleVariableRef CVarEdGraphStripNodeComment(
	TEXT("cook.StripGraphNodeComments"),
	GEdGraphStripNodeComment,
	TEXT("1 = Strip graph node comments on cook. 0 = off"));

FEdGraphTerminalType FEdGraphTerminalType::FromPinType(const FEdGraphPinType& PinType)
{
	FEdGraphTerminalType TerminalType;
	TerminalType.TerminalCategory = PinType.PinCategory;
	TerminalType.TerminalSubCategory = PinType.PinSubCategory;
	TerminalType.TerminalSubCategoryObject = PinType.PinSubCategoryObject;
	TerminalType.bTerminalIsConst = PinType.bIsConst;
	TerminalType.bTerminalIsWeakPointer = PinType.bIsWeakPointer;
	TerminalType.bTerminalIsUObjectWrapper = PinType.bIsUObjectWrapper;
	return TerminalType;
}

FArchive& operator<<(FArchive& Ar, FEdGraphTerminalType& T)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::PinsStoreFName)
	{
		Ar << T.TerminalCategory;
		Ar << T.TerminalSubCategory;
	}
	else
	{
		FString TerminalCategoryStr;
		Ar << TerminalCategoryStr;

		if (Ar.UEVer() < VER_UE4_ADDED_SOFT_OBJECT_PATH)
		{
			// Handle asset->soft object rename, this is here instead of BP code because this structure is embedded
			if (TerminalCategoryStr == TEXT("asset"))
			{
				TerminalCategoryStr = TEXT("softobject");
			}
			else if (TerminalCategoryStr == TEXT("assetclass"))
			{
				TerminalCategoryStr = TEXT("softclass");
			}
		}

		T.TerminalCategory = *TerminalCategoryStr;

		FString TerminalSubCategoryStr;
		Ar << TerminalSubCategoryStr;
		T.TerminalSubCategory = *TerminalSubCategoryStr;
	}

	// See: FArchive& operator<<( FArchive& Ar, FWeakObjectPtr& WeakObjectPtr )
	// The PinSubCategoryObject should be serialized into the package.
	if (!Ar.IsObjectReferenceCollector() || Ar.IsModifyingWeakAndStrongReferences() || Ar.IsPersistent())
	{
		UObject* Object = T.TerminalSubCategoryObject.Get(true);
		Ar << Object;
		if (Ar.IsLoading() || Ar.IsModifyingWeakAndStrongReferences())
		{
			T.TerminalSubCategoryObject = Object;
		}
	}

	Ar << T.bTerminalIsConst;
	Ar << T.bTerminalIsWeakPointer;

	if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::PinTypeIncludesUObjectWrapperFlag)
	{
		Ar << T.bTerminalIsUObjectWrapper;
	}

	if (Ar.IsLoading())
	{
		bool bFixupPinCategories =
			(Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::BlueprintPinsUseRealNumbers) &&
			((T.TerminalCategory == TEXT("double")) || (T.TerminalCategory == TEXT("float")));

		if (bFixupPinCategories)
		{
			T.TerminalCategory = TEXT("real");
			T.TerminalSubCategory = TEXT("double");
		}
	}

	return Ar;
}

FName const FNodeMetadata::DefaultGraphNode(TEXT("DefaultGraphNode"));

#if WITH_EDITOR
UEdGraphNode::FCreatePinParams::FCreatePinParams(const FEdGraphPinType& PinType)
	: ContainerType(PinType.ContainerType)
	, bIsReference(PinType.bIsReference)
	, bIsConst(PinType.bIsConst)
	, Index(INDEX_NONE)
	, ValueTerminalType(PinType.PinValueType)
{
}
#endif // WITH_EDITOR

/////////////////////////////////////////////////////
// UGraphNodeContextMenuContext

UGraphNodeContextMenuContext::UGraphNodeContextMenuContext() :
	Pin(nullptr)
{
}

void UGraphNodeContextMenuContext::Init(const UEdGraph* InGraph, const UEdGraphNode* InNode, const UEdGraphPin* InPin, bool bInDebuggingMode)
{
	Graph = InGraph;
	Node = InNode;
	Pin = InPin;
	bIsDebugging = bInDebuggingMode;

#if WITH_EDITOR
	Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
#endif

	if (Pin)
	{
		Node = Pin->GetOwningNode();
	}
}

/////////////////////////////////////////////////////
// UEdGraphNode

#if WITH_EDITORONLY_DATA
void GatherGraphNodeForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	const UEdGraphNode* const GraphNode = CastChecked<UEdGraphNode>(Object);
	GraphNode->GatherForLocalization(PropertyLocalizationDataGatherer, GatherTextFlags);
}

void UEdGraphNode::GatherForLocalization(FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags) const
{
	// We need to gather graph pins separately as they're no longer UObjects or UProperties
	// plus they have custom logic for working out whether they're the default value
	const FString PathToObject = GetPathName();
	for (const UEdGraphPin* Pin : Pins)
	{
		if (!Pin->DefaultTextValue.IsEmpty())
		{
			if (Pin->DoesDefaultValueMatchAutogenerated())
			{
				PropertyLocalizationDataGatherer.MarkDefaultTextInstance(Pin->DefaultTextValue);
			}
			else
			{
				PropertyLocalizationDataGatherer.GatherTextInstance(Pin->DefaultTextValue, FString::Printf(TEXT("%s.%s"), *PathToObject, *Pin->GetName()), /*bIsEditorOnly*/true);
			}
		}
	}

	PropertyLocalizationDataGatherer.GatherLocalizationDataFromObject(this, GatherTextFlags);
}
#endif

UEdGraphNode::UEdGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AdvancedPinDisplay(ENodeAdvancedPins::NoPins)
	, EnabledState(ENodeEnabledState::Enabled)
	, bUserSetEnabledState(false)
#if WITH_EDITORONLY_DATA
	, bIsNodeEnabled_DEPRECATED(true)
	, bCanResizeNode(false)
	, bUnrelated(false)
	, bCommentBubblePinned(false)
	, bCommentBubbleVisible(false)
	, bCommentBubbleMakeVisible(false)
#endif // WITH_EDITORONLY_DATA

{
#if WITH_EDITORONLY_DATA
	{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(UEdGraphNode::StaticClass(), &GatherGraphNodeForLocalization); }
#endif
}

void UEdGraphNode::Serialize(FArchive& Ar)
{
#if WITH_EDITOR
	Ar.UsingCustomVersion(FBlueprintsObjectVersion::GUID);
#endif

	FString StrippedNodeComment;
	if (GEdGraphStripNodeComment && Ar.IsCooking())
	{
		StrippedNodeComment = NodeComment;
		NodeComment = FString();
	}

	Super::Serialize(Ar);

	if (StrippedNodeComment.Len() > 0)
	{
		NodeComment = StrippedNodeComment;
	}

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		// If this was an older version, ensure that we update the enabled state for already-disabled nodes.
		// Note: We need to do this here and not in PostLoad() as it must be assigned prior to compile-on-load.
		if (!bIsNodeEnabled_DEPRECATED && !bUserSetEnabledState && EnabledState == ENodeEnabledState::Enabled)
		{
			EnabledState = ENodeEnabledState::Disabled;
		}

		if (Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
		{
			if (Ar.CustomVer(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::EdGraphPinOptimized)
			{
				for (UEdGraphPin_Deprecated* LegacyPin : DeprecatedPins)
				{
					Ar.Preload(LegacyPin);
					if (UEdGraphPin::FindPinCreatedFromDeprecatedPin(LegacyPin) == nullptr)
					{
						UEdGraphPin::CreatePinFromDeprecatedPin(LegacyPin);
					}
				}
			}
		}
	}

	if (Ar.CustomVer(FBlueprintsObjectVersion::GUID) >= FBlueprintsObjectVersion::EdGraphPinOptimized)
	{
		UEdGraphPin::SerializeAsOwningNode(Ar, Pins);
	}
#endif
}

#if WITH_EDITORONLY_DATA
void UEdGraphNode::DeclareCustomVersions(FArchive& Ar, const UClass* SpecificSubclass)
{
	Super::DeclareCustomVersions(Ar, SpecificSubclass);
	UEdGraphPin::DeclarePinCustomVersions(Ar);
}
#endif


bool UEdGraphNode::GetCanRenameNode() const
{
#if WITH_EDITORONLY_DATA
	return bCanRenameNode;
#else
	return false;
#endif
}

#if WITH_EDITOR

FString UEdGraphNode::GetPropertyNameAndValueForDiff(const FProperty* Prop, const uint8* PropertyAddr) const
{
	FString ExportedStringValue;
	if (const FFloatProperty* FloatProp = CastField<const FFloatProperty>(Prop))
	{
		// special case for floats to remove unnecessary zeros
		const float FloatValue = FloatProp->GetPropertyValue(PropertyAddr);
		ExportedStringValue = FString::SanitizeFloat(FloatValue);
	}
	else
	{
		Prop->ExportTextItem_Direct(ExportedStringValue, PropertyAddr, NULL, NULL, PPF_PropertyWindow, NULL);
	}

	const bool bIsBool = Prop->IsA(FBoolProperty::StaticClass());
	return FString::Printf(TEXT("%s: %s"), *FName::NameToDisplayString(Prop->GetName(), bIsBool), *ExportedStringValue);
}


void UEdGraphNode::DiffProperties(UClass* StructA, UClass* StructB, UObject* DataA, UObject* DataB, FDiffResults& Results, FDiffSingleResult& Diff) const
{
	// Find the common parent class in case the other node isn't of the same type
	UClass* ClassToViewAs = StructA;
	while (!DataB->IsA(ClassToViewAs))
	{
		ClassToViewAs = ClassToViewAs->GetSuperClass();
	}

	DiffProperties(ClassToViewAs, ClassToViewAs, (uint8*)DataA, (uint8*)DataB, Results, Diff);
}

void UEdGraphNode::DiffProperties(UStruct* StructA, UStruct* StructB, uint8* DataA, uint8* DataB, FDiffResults& Results, FDiffSingleResult& Diff) const
{
	// Run through all the properties in the first struct
	for (TFieldIterator<FProperty> PropertyIt(StructA, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Prop = *PropertyIt;
		FProperty* PropB = StructB->FindPropertyByName(Prop->GetFName());

		if (!PropB || Prop->GetClass() != PropB->GetClass())
		{
			// Skip if properties don't match
			continue;
		}

		auto ShouldSkipProperty = [](const FProperty& Prop)
			{
				// if the property is an inner property (e.g. for a container) then use the outer property for visibility:
				const FProperty* PropertyToTest = &Prop;
				const FProperty* Owner = Prop.GetOwnerProperty();
				if(Owner)
				{
					PropertyToTest = Owner;
				}

				return !PropertyToTest->HasAnyPropertyFlags(CPF_Edit | CPF_BlueprintVisible) ||
					PropertyToTest->HasAnyPropertyFlags(CPF_Transient) ||
					PropertyToTest->HasAnyPropertyFlags(CPF_DisableEditOnInstance) ||
					PropertyToTest->IsA(FDelegateProperty::StaticClass()) ||
					PropertyToTest->IsA(FMulticastDelegateProperty::StaticClass());
			};

		// skip properties we cant see
		if (ShouldSkipProperty(*Prop))
		{
			continue;
		}

		DiffUtils::FDiffParameters ComparisonParameters;
		ComparisonParameters.ShouldIgnorePropertyPredicate = ShouldSkipProperty;
		TArray<FPropertySoftPath> DifferingSubProperties;

		if (!DiffUtils::Identical(FResolvedProperty(DataA, Prop), FResolvedProperty(DataB, PropB), StructA, StructB, ComparisonParameters, DifferingSubProperties))
		{
			// Only bother setting up the display data if we're storing the result
			if (Results.CanStoreResults())
			{
				TStringBuilder<256> StringBuilder;
				for (int DifferingIndex = 0; DifferingIndex < DifferingSubProperties.Num(); DifferingIndex++)
				{
					StringBuilder += DifferingSubProperties[DifferingIndex].ToDisplayName();
				}

				if (StringBuilder.Len() > 0)
				{
					Diff.DisplayString = FText::Format(LOCTEXT("DIF_NodePropertyWithSubPropertiesFmt", "Property Changed: {0} - {1}"),
						FText::FromString(Prop->GetName()),
						FText::FromString(StringBuilder.ToString()));
				}
				else
				{
					Diff.DisplayString = FText::Format(LOCTEXT("DIF_NodePropertyFmt", "Property Changed: {0}"), FText::FromString(Prop->GetName()));
				}
			}
			Results.Add(Diff);
		}
	}
}

UEdGraphPin* UEdGraphNode::CreatePin(EEdGraphPinDirection Dir, const FEdGraphPinType& InPinType, const FName PinName, int32 Index /*= INDEX_NONE*/)
{
	UEdGraphPin* NewPin = UEdGraphPin::CreatePin(this);
	NewPin->PinName = PinName;
	NewPin->Direction = Dir;

	NewPin->PinType = InPinType;

	Modify(false);
	if (Pins.IsValidIndex(Index))
	{
		Pins.Insert(NewPin, Index);
	}
	else
	{
		Pins.Add(NewPin);
	}
	return NewPin;
}

UEdGraphPin* UEdGraphNode::CreatePin(const EEdGraphPinDirection Dir, const FName PinCategory, const FName PinSubCategory, UObject* PinSubCategoryObject, const FName PinName, const FCreatePinParams& PinParams)
{
	FEdGraphPinType PinType(PinCategory, PinSubCategory, PinSubCategoryObject, PinParams.ContainerType, PinParams.bIsReference, PinParams.ValueTerminalType);
	PinType.bIsConst = PinParams.bIsConst;

	return CreatePin(Dir, PinType, PinName, PinParams.Index);
}

UEdGraphPin* UEdGraphNode::FindPin(const FName PinName, const EEdGraphPinDirection Direction) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if ((Direction == EGPD_MAX || Direction == Pin->Direction) && Pin->PinName == PinName)
		{
			return Pin;
		}
	}

	return nullptr;
}

UEdGraphPin* UEdGraphNode::FindPin(const TCHAR* const PinName, const EEdGraphPinDirection Direction) const
{
	const FName PinFName(PinName, FNAME_Find);
	return (!PinFName.IsNone() ? FindPin(PinFName, Direction) : nullptr);
}

UEdGraphPin* UEdGraphNode::FindPinById(const FGuid PinId) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinId == PinId)
		{
			return Pin;
		}
	}

	return nullptr;
}

UEdGraphPin* UEdGraphNode::FindPinByIdChecked(const FGuid PinId) const
{
	UEdGraphPin* Result = FindPinById(PinId);
	check(Result);
	return Result;
}

UEdGraphPin* UEdGraphNode::FindPinByPredicate(TFunctionRef<bool(UEdGraphPin* InPin)> InFunction) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (InFunction(Pin))
		{
			return Pin;
		}
	}

	return nullptr;
}

bool UEdGraphNode::RemovePin(UEdGraphPin* Pin)
{
	check( Pin );
	
	Modify();
	UEdGraphPin* RootPin = (Pin->ParentPin != nullptr) ? Pin->ParentPin : Pin;
	RootPin->MarkAsGarbage();

	if (Pins.Remove( RootPin ))
	{
		// Remove any children pins to ensure the entirety of the pin's representation is removed
		for (UEdGraphPin* ChildPin : RootPin->SubPins)
		{
			Pins.Remove(ChildPin);
			ChildPin->MarkAsGarbage();
		}
		OnPinRemoved(Pin);
		return true;
	}

	return false;
}

void UEdGraphNode::BreakAllNodeLinks(bool bAlwaysMarkDirty)
{
	TSet<UEdGraphNode*> NodeList;

	NodeList.Add(this);

	// Iterate over each pin and break all links
	for (int32 PinIdx = 0; PinIdx < Pins.Num(); ++PinIdx)
	{
		if (UEdGraphPin* Pin = Pins[PinIdx])
		{
			// Save all the connected nodes to be notified below
			for (UEdGraphPin* Connection : Pin->LinkedTo)
			{
				if (Connection != nullptr)
				{
					NodeList.Add(Connection->GetOwningNode());
				}
			}

			constexpr bool bNotifyNodes = false;
			Pin->BreakAllPinLinks(bNotifyNodes, bAlwaysMarkDirty);
		}
	}

	// Send a notification to all nodes that lost a connection
	for (UEdGraphNode* Node : NodeList)
	{
		Node->NodeConnectionListChanged();
	}
}

void UEdGraphNode::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	ensure(Pin.GetOwningNode() == this);
	HoverTextOut = Pin.PinToolTip;
}

void UEdGraphNode::SnapToGrid(uint32 GridSnapSize)
{
	NodePosX = GridSnapSize * (NodePosX / GridSnapSize);
	NodePosY = GridSnapSize * (NodePosY / GridSnapSize);
}

bool UEdGraphNode::ShowVisualWarning() const
{
	return false;
}

FText UEdGraphNode::GetVisualWarningTooltipText() const
{
	return FText();
}

class UEdGraph* UEdGraphNode::GetGraph() const
{
	UEdGraph* Graph = Cast<UEdGraph>(GetOuter());
	if (Graph == nullptr && IsValidChecked(this))
	{
		if(GetOuter() != GetTransientPackage())
		{
			ensureMsgf(false, TEXT("EdGraphNode::GetGraph : '%s' does not have a UEdGraph as an Outer."), *GetPathName());
		}
	}
	return Graph;
}

void UEdGraphNode::DestroyNode()
{
	UEdGraph* ParentGraph = GetGraph();
	check(ParentGraph);

	// Remove the node - this will break all links. Will be GC'd after this.
	ParentGraph->RemoveNode(this);
}

void UEdGraphNode::RemovePinAt(const int32 PinIndex, const EEdGraphPinDirection PinDirection)
{
	Modify();

	UEdGraphPin* OldPin = GetPinWithDirectionAt(PinIndex, PinDirection);
	checkf(OldPin, TEXT("Tried to remove a non-existent pin."));

	OldPin->BreakAllPinLinks();
	RemovePin(OldPin);

	GetGraph()->NotifyGraphChanged();
}

const class UEdGraphSchema* UEdGraphNode::GetSchema() const
{
	UEdGraph* ParentGraph = GetGraph();
	return ParentGraph ? ParentGraph->GetSchema() : NULL;
}

bool UEdGraphNode::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	return CanCreateUnderSpecifiedSchema(Graph->GetSchema());
}

FLinearColor UEdGraphNode::GetNodeTitleColor() const
{
	return FLinearColor(0.4f, 0.62f, 1.0f);
}

FLinearColor UEdGraphNode::GetNodeTitleTextColor() const
{
	FLinearColor ReturnTextColor = FLinearColor::White;
	if (!IsNodeEnabled() || IsDisplayAsDisabledForced() || IsNodeUnrelated())
	{
		ReturnTextColor *= FLinearColor(1.0f, 1.0f, 1.0f, 0.3f);
	}
	return ReturnTextColor;
}

FLinearColor UEdGraphNode::GetNodeCommentColor() const
{
	return FLinearColor::White;
}

FLinearColor UEdGraphNode::GetNodeBodyTintColor() const
{
	return FLinearColor::White;
}

FText UEdGraphNode::GetTooltipText() const
{
	return GetClass()->GetToolTipText();
}

FString UEdGraphNode::GetDocumentationExcerptName() const
{
	// Default the node to searching for an excerpt named for the C++ node class name, including the U prefix.
	// This is done so that the excerpt name in the doc file can be found by find-in-files when searching for the full class name.
	UClass* MyClass = GetClass();
	return FString::Printf(TEXT("%s%s"), MyClass->GetPrefixCPP(), *MyClass->GetName());
}

FSlateIcon UEdGraphNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Default_16x");
	return Icon;
}

FString UEdGraphNode::GetDescriptiveCompiledName() const
{
	return GetFName().GetPlainNameString();
}

void UEdGraphNode::ResizeNode(const FVector2f& NewSize)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ResizeNode(FVector2D(NewSize));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UEdGraphNode::IsDeprecated() const
{
	return GetClass()->HasAnyClassFlags(CLASS_Deprecated);
}

FEdGraphNodeDeprecationResponse UEdGraphNode::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response;

	if (DeprecationType == EEdGraphNodeDeprecationType::NodeTypeIsDeprecated)
	{
		Response.MessageType = EEdGraphNodeDeprecationMessageType::Warning;
		Response.MessageText = NSLOCTEXT("EdGraphCompiler", "NodeDeprecated_Warning", "@@ is deprecated; please replace or remove it.");
	}
	else if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		Response.MessageType = EEdGraphNodeDeprecationMessageType::Warning;
		Response.MessageText = NSLOCTEXT("EdGraphCompiler", "NodeDeprecatedReference_Note", "@@ has a deprecated reference; please replace or remove it.");
	}

	return Response;
}

void UEdGraphNode::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UEdGraphNode* This = CastChecked<UEdGraphNode>(InThis);
	for (UEdGraphPin* Pin : This->Pins)
	{
		if (Pin)
		{
			Pin->AddStructReferencedObjects(Collector);
		}
	}
}

void UEdGraphNode::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

#if WITH_EDITORONLY_DATA
	if (!NodeUpgradeMessage.IsEmpty())
	{
		// When saving, we clear any upgrade messages
		NodeUpgradeMessage = FText::GetEmpty();
	}
#endif // WITH_EDITORONLY_DATA
}

void UEdGraphNode::PostLoad()
{
	Super::PostLoad();

	// Create Guid if not present (and not CDO)
	if(!NodeGuid.IsValid() && !IsTemplate() && GetLinker() && GetLinker()->IsPersistent() && GetLinker()->IsLoading())
	{
		UE_LOGF(LogBlueprint, Warning, "Node '%ls' missing NodeGuid, this can cause deterministic cooking issues please resave package.", *GetPathName());

		// Generate new one
		CreateNewGuid();
	}

	// Duplicating a Blueprint needs to have a new Node Guid generated, which was not occuring before this version
	if(GetLinkerUEVersion() < VER_UE4_POST_DUPLICATE_NODE_GUID)
	{
		UE_LOGF(LogBlueprint, Warning, "Node '%ls' missing NodeGuid because of upgrade from old package version, this can cause deterministic cooking issues please resave package.", *GetPathName());

		// Generate new one
		CreateNewGuid();
	}
	// Moving to the new style comments requires conversion to preserve previous state
	if(GetLinkerUEVersion() < VER_UE4_GRAPH_INTERACTIVE_COMMENTBUBBLES)
	{
		bCommentBubbleVisible = !NodeComment.IsEmpty();
	}

	if (DeprecatedPins.Num())
	{
		for (UEdGraphPin_Deprecated* LegacyPin : DeprecatedPins)
		{
			LegacyPin->Rename(nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch | REN_NonTransactional);
			LegacyPin->SetFlags(RF_Transient);
			LegacyPin->MarkAsGarbage();
		}

		DeprecatedPins.Empty();
	}

}

void UEdGraphNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(const UEdGraphSchema* Schema = GetSchema())
	{
		Schema->ForceVisualizationCacheClear();
	}
}

void UEdGraphNode::PostEditUndo()
{
	UEdGraphPin::ResolveAllPinReferences();
	
	return UObject::PostEditUndo();
}

void UEdGraphNode::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	Super::ExportCustomProperties(Out, Indent);

	for (const UEdGraphPin* Pin : Pins)
	{
		FString PinString;
		Pin->ExportTextItem(PinString, PPF_Delimited);
		Out.Logf(TEXT("%sCustomProperties Pin %s\r\n"), FCString::Spc(Indent), *PinString);
	}
}

void UEdGraphNode::ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn)
{
	Super::ImportCustomProperties(SourceText, Warn);

	if (FParse::Command(&SourceText, TEXT("Pin")))
	{
		UEdGraphPin* NewPin = UEdGraphPin::CreatePin(this);
		const bool bParseSuccess = NewPin->ImportTextItem(SourceText, PPF_Delimited, this, GWarn);
		if (bParseSuccess)
		{
			Pins.Add(NewPin);
		}
		else
		{
			// Still adding a nullptr to preserve indices
			Pins.Add(nullptr);
		}
	}
}

void UEdGraphNode::BeginDestroy()
{
	for (UEdGraphPin* Pin : Pins)
	{
		Pin->MarkAsGarbage();
	}

	Pins.Empty();

	Super::BeginDestroy();
}

void UEdGraphNode::CreateNewGuid()
{
	NodeGuid = FGuid::NewGuid();
}

#if WITH_EDITORONLY_DATA
void UEdGraphNode::CreateDeterministicGuid()
{
	NodeGuid = FGuid::NewDeterministicGuid(GetPathName());
}
#endif//WITH_EDITORONLY_DATA

void UEdGraphNode::FindDiffs(UEdGraphNode* OtherNode, struct FDiffResults& Results)
{
	if (OtherNode != nullptr)
	{
		FDiffSingleResult Diff;
		Diff.Diff = EDiffType::NODE_PROPERTY;
		Diff.Node1 = this;
		Diff.Node2 = OtherNode;
		Diff.ToolTip = LOCTEXT("DIF_NodePropertyToolTip", "A Property of the node has changed");
		Diff.Category = EDiffType::MODIFICATION;

		// Diff the properties between the nodes
		DiffProperties(GetClass(), OtherNode->GetClass(), this, OtherNode, Results, Diff);
	}
}

void UEdGraphNode::DestroyPin(UEdGraphPin* Pin)
{
	Pin->MarkAsGarbage();
}

bool UEdGraphNode::CanDuplicateNode() const
{
	return true;
}

bool UEdGraphNode::CanUserDeleteNode() const
{
	return true;
}

FText UEdGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString(GetClass()->GetName());
}

FString UEdGraphNode::GetFindReferenceSearchString() const
{
	return GetNodeTitle(ENodeTitleType::ListView).ToString();
}

FString UEdGraphNode::GetFindReferenceSearchString(EGetFindReferenceSearchStringFlags InFlags) const
{
	return GetFindReferenceSearchString_Impl(InFlags);
}

UObject* UEdGraphNode::GetJumpTargetForDoubleClick() const
{
	return nullptr;
}

bool UEdGraphNode::CanJumpToDefinition() const
{
	return false;
}

void UEdGraphNode::JumpToDefinition() const
{
	// No implementation in the base graph node
}

FText UEdGraphNode::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	return GetSchema()->GetPinDisplayName(Pin);
}

int32 UEdGraphNode::GetPinIndex(UEdGraphPin* Pin) const
{
	return Pins.Find(Pin);
}

bool UEdGraphNode::ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const
{
	OutInputPinIndex = -1;
	OutOutputPinIndex = -1; 
	return false;
}


UEdGraphPin* UEdGraphNode::GetPinAt(int32 index) const
{
	if (Pins.Num() > index)
	{
		return Pins[index];
	}
	return nullptr;
}

UEdGraphPin* UEdGraphNode::GetPinWithDirectionAt(int32 PinIndex, EEdGraphPinDirection PinDirection) const
{
	// Map requested input to actual pin index
	int32 MatchingPinCount = 0;
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == PinDirection)
		{
			if (PinIndex == MatchingPinCount)
			{
				return Pin;
			}
			++MatchingPinCount;
		}
	}

	return nullptr;
}

void UEdGraphNode::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	if (GetSchema() == nullptr)
	{
		ensure(false);
		return;
	}

	// Searchable - Primary label for the item in the search results
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_Name, GetNodeTitle(ENodeTitleType::ListView)));

	// Searchable - As well as being searchable, this displays in the tooltip for the node
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_ClassName, FText::FromString(GetClass()->GetName())));

	// Non-searchable - Used to lookup the node when attempting to jump to it
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NodeGuid, FText::FromString(NodeGuid.ToString(EGuidFormats::Digits))));

	// Non-searchable - Important for matching pin types with icons and colors, stored here so that each pin does not store it
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_SchemaName, FText::FromString(GetSchema()->GetClass()->GetName())));

	// Non-Searchable - Used to display the icon and color for this node for better visual identification.
	FLinearColor GlyphColor = FLinearColor::White;
	FSlateIcon Icon = GetIconAndTint(GlyphColor);
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_Glyph, FText::FromName(Icon.GetStyleName())));
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_GlyphStyleSet, FText::FromName(Icon.GetStyleSetName())));
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_GlyphColor, FText::FromString(GlyphColor.ToString())));
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_Comment, FText::FromString(NodeComment)));
}

void UEdGraphNode::AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	// Searchable - Primary label for the item in the search results
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_Name, Pin->GetSchema()->GetPinDisplayName(Pin)));

	// Searchable - The pin's default value as a text string
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_DefaultValue, Pin->GetDefaultAsText()));
}

void UEdGraphNode::OnUpdateCommentText( const FString& NewComment )
{
	if( !NodeComment.Equals( NewComment ))
	{
		const FScopedTransaction Transaction( LOCTEXT( "CommentCommitted", "Comment Changed" ) );
		Modify();
		NodeComment	= NewComment;
	}
}

FText UEdGraphNode::GetKeywords() const
{
	return GetClass()->GetMetaDataText(TEXT("Keywords"), TEXT("UObjectKeywords"), GetClass()->GetFullGroupName(false));
}

void UEdGraphNode::AddNodeUpgradeNote(FText InUpgradeNote)
{
#if WITH_EDITORONLY_DATA
	if (NodeUpgradeMessage.IsEmpty())
	{
		NodeUpgradeMessage = InUpgradeNote;
	}
	else
	{
		NodeUpgradeMessage = FText::Format(FText::FromString(TEXT("{0}\n{1}")), NodeUpgradeMessage, InUpgradeNote);
	}
#endif
}

bool UEdGraphNode::ShouldMakeCommentBubbleVisible() const
{
	return bCommentBubbleMakeVisible;
}

void UEdGraphNode::SetMakeCommentBubbleVisible(bool MakeVisible)
{
	bCommentBubbleMakeVisible = MakeVisible;
}

void UEdGraphNode::ForEachNodeDirectlyConnected(TFunctionRef<void(UEdGraphNode*)> Func)
{
	TSet<UEdGraphNode*> DirectNeighbors;
	for( UEdGraphPin* Pin : Pins )
	{
		if(Pin->LinkedTo.Num() > 0)
		{
			for(UEdGraphPin* Connection : Pin->LinkedTo)
			{
				// avoid including the current node in the case of a self connection:
				if(Connection->GetOwningNode() != this)
				{
					DirectNeighbors.Add(Connection->GetOwningNode());
				}
			}
		}
	}

	for(UEdGraphNode* Neighbor : DirectNeighbors)
	{
		Func(Neighbor);
	}
}

void UEdGraphNode::ForEachNodeDirectlyConnectedIf(TFunctionRef<bool(const UEdGraphPin* Pin)> Filter, TFunctionRef<void(UEdGraphNode*)> Func)
{
	TSet<UEdGraphNode*> NeighborsAcceptedForConsideration;
	for( UEdGraphPin* Pin : Pins )
	{
		if(Pin->LinkedTo.Num() > 0 && Filter(Pin))
		{
			for(UEdGraphPin* Connection : Pin->LinkedTo)
			{
				// avoid including the current node in the case of a self connection:
				if(Connection->GetOwningNode() != this)
				{
					NeighborsAcceptedForConsideration.Add(Connection->GetOwningNode());
				}
			}
		}
	}

	for(UEdGraphNode* Neighbor : NeighborsAcceptedForConsideration)
	{
		Func(Neighbor);
	}
}

void UEdGraphNode::ForEachNodeDirectlyConnectedToInputs(TFunctionRef<void(UEdGraphNode*)> Func)
{
	ForEachNodeDirectlyConnectedIf(
		[](const UEdGraphPin* Pin)
		{ 
			if(Pin->Direction == EGPD_Input)
			{
				return true;
			}
			return false;
		},
		Func
	);
}

void UEdGraphNode::ForEachNodeDirectlyConnectedToOutputs(TFunctionRef<void(UEdGraphNode*)> Func)
{
	ForEachNodeDirectlyConnectedIf(
		[](const UEdGraphPin* Pin)
		{ 
			if(Pin->Direction == EGPD_Output)
			{
				return true;
			}
			return false;
		},
		Func
	);
}

bool UEdGraphNode::IsSnappedToAny(const TSet<TObjectPtr<UObject>>& NodeSet ) const
{
	for (const UEdGraphPin* Pin : Pins)
	{
		for (UEdGraphPin* Connection : Pin->LinkedTo)
		{
			if(Connection->IsSnappedParentOf(this))
			{
				if( NodeSet.Contains(Connection->GetOwningNode()))
				{
					return true;
				}
				else if(Connection->GetOwningNode()->IsSnappedToAny(NodeSet))
				{
					return true;
				}
			}
		}
	}
	return false;
}

UEdGraphNode* UEdGraphNode::GetRootSnappedParent(const TSet<TObjectPtr<UObject>>& NodeSet) const
{
	for (const UEdGraphPin* Pin : Pins)
	{
		for (UEdGraphPin* Connection : Pin->LinkedTo)
		{
			if(Connection->IsSnappedParentOf(this))
			{
				UEdGraphNode* OwningNode = Connection->GetOwningNode();
				if( NodeSet.Contains(OwningNode))
				{
					return OwningNode;
				}
				OwningNode = OwningNode->GetRootSnappedParent(NodeSet);
				if(OwningNode)
				{
					return OwningNode;
				}
			}
		}
	}
	return nullptr;
}

#endif	//#if WITH_EDITOR

bool UEdGraphNode::IsInDevelopmentMode() const
{
#if WITH_EDITOR
	// By default, development mode is implied when running in the editor and not cooking via commandlet, unless enabled in the project settings.
	return !IsRunningCookCommandlet() || GetDefault<UCookerSettings>()->bCompileBlueprintsInDevelopmentMode;
#else
	return false;
#endif
}

bool UEdGraphNode::IsAutomaticallyPlacedGhostNode() const
{
	return !bUserSetEnabledState && (EnabledState == ENodeEnabledState::Disabled);
}

void UEdGraphNode::MakeAutomaticallyPlacedGhostNode()
{
	EnabledState = ENodeEnabledState::Disabled;
	NodeComment = LOCTEXT("DisabledNodeComment", "This node is disabled and will not be called.\nDrag off pins to build functionality.").ToString();
	bUserSetEnabledState = false;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
