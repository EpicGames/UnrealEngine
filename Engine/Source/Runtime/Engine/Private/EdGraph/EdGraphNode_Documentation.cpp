// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/EdGraphNode_Documentation.h"
#include "Textures/SlateIcon.h"

#if WITH_EDITOR
#include "Layout/SlateRect.h"
#include "Kismet2/Kismet2NameValidators.h"
#endif

#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphNode_Documentation)

#define LOCTEXT_NAMESPACE "EdGraph"

/////////////////////////////////////////////////////
// UEdGraphNode_Documentation

UEdGraphNode_Documentation::UEdGraphNode_Documentation( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
#if WITH_EDITORONLY_DATA
	bCanResizeNode = true;
	bCanRenameNode = false;
#endif // WITH_EDITORONLY_DATA
	Link = TEXT( "Shared/GraphNodes/Blueprint" );
	Excerpt = TEXT( "UEdGraphNode_Documentation" );
}

#if WITH_EDITOR
void UEdGraphNode_Documentation::PostPlacedNewNode()
{
	NodeComment = TEXT( "" );
}

FText UEdGraphNode_Documentation::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "DocumentationBlock_Tooltip", "UDN Documentation Excerpt");
}

FText UEdGraphNode_Documentation::GetNodeTitle( ENodeTitleType::Type TitleType ) const
{
	FText NodeTitle;
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		NodeTitle = NSLOCTEXT("K2Node", "DocumentationBlock_ListTitle", "Add Documentation Node...");
	}
	else
	{
		NodeTitle = NSLOCTEXT("K2Node", "DocumentationBlock_Title", "UDN Documentation Excerpt");
	}

	return NodeTitle;
}

FSlateIcon UEdGraphNode_Documentation::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation");
	return Icon;
}

FText UEdGraphNode_Documentation::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	return GetNodeTitle( ENodeTitleType::ListView );
}

void UEdGraphNode_Documentation::ResizeNode(const FVector2f& NewSize)
{
	if( bCanResizeNode ) 
	{
		NodeHeight = UE::LWC::FloatToIntCastChecked<int32>(NewSize.Y);
		NodeWidth = UE::LWC::FloatToIntCastChecked<int32>(NewSize.X);
	}
}

void UEdGraphNode_Documentation::SetBounds( const class FSlateRect& Rect )
{
	NodePosX = Rect.Left;
	NodePosY = Rect.Top;

	FVector2f Size = Rect.GetSize();
	NodeWidth = Size.X;
	NodeHeight = Size.Y;
}

void UEdGraphNode_Documentation::OnRenameNode( const FString& NewName )
{
	NodeComment = NewName;
}

TSharedPtr<class INameValidatorInterface> UEdGraphNode_Documentation::MakeNameValidator() const
{
	// Documentation can be duplicated, etc...
	return MakeShareable( new FDummyNameValidator( EValidatorResult::Ok ));
}

#endif // WITH_EDITOR

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

