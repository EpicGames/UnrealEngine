// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/EdGraphPin.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/TextProperty.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphPin)
#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Editor/Transactor.h"
#include "Misc/ConfigCacheIni.h"
#include "TickableEditorObject.h"
#else
#include "HAL/IConsoleManager.h"
#include "Tickable.h"
#endif

#define LOCTEXT_NAMESPACE "EdGraph"

#if WITH_EDITOR
extern UNREALED_API UEditorEngine* GEditor;
#endif

class FPinDeletionQueue : 
#if WITH_EDITOR
	public FTickableEditorObject
#else
	public FTickableGameObject
#endif
{
public:

	static FPinDeletionQueue* Get()
	{
		static FPinDeletionQueue* Instance = new FPinDeletionQueue;
		return Instance;
	}

	static void Add(UEdGraphPin* PinToDelete)
	{
		FPinDeletionQueue* PinDeletionQueue = Get();
		FWriteScopeLock ScopeLock(PinDeletionQueue->PinsToDeleteLock);
		PinDeletionQueue->PinsToDelete.Add(PinToDelete);
	}

	virtual void Tick(float DeltaTime) override
	{
		TArray<UEdGraphPin*> LocalPinsToDelete;
		{
			FWriteScopeLock ScopeLock(PinsToDeleteLock);
			Swap(LocalPinsToDelete, PinsToDelete);
		}

		for (UEdGraphPin* Pin : LocalPinsToDelete)
		{
			delete Pin;
		}
	}

	virtual bool IsTickable() const override
	{
		FReadScopeLock ScopeLock(PinsToDeleteLock);
		return (PinsToDelete.Num() > 0);
	}

	/** return the stat id to use for this tickable **/
	virtual TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FPinDeletionQueue, STATGROUP_Tickables);
	}

	virtual ~FPinDeletionQueue() = default;

private:

	FPinDeletionQueue() = default;
	mutable FRWLock PinsToDeleteLock;
	TArray<UEdGraphPin*> PinsToDelete;

};

struct FGraphConnectionSanitizer
{
	void AddCorruptGraph(UEdGraph* CorruptGraph);
	void SanitizeConnectionsForCorruptGraphs();

	TArray<UEdGraph*> CorruptGraphs;
} GraphConnectionSanitizer;

void FGraphConnectionSanitizer::AddCorruptGraph(UEdGraph* CorruptGraph)
{
	CorruptGraphs.Add(CorruptGraph);
}

void FGraphConnectionSanitizer::SanitizeConnectionsForCorruptGraphs()
{
	for(UEdGraph* CorruptGraph : CorruptGraphs)
	{
		// validate all links for nodes in this graph:
		for(UEdGraphNode* Node : CorruptGraph->Nodes)
		{
			TArray<UEdGraphPin*>& NodePins = Node->Pins;
			for(int32 I = NodePins.Num() - 1; I >= 0; --I )
			{
				UEdGraphPin* CurrentPin = NodePins[I];
				// check for null or assymetry:
				if(CurrentPin == nullptr)
				{
					NodePins.RemoveAtSwap(I);
				}
				else
				{
					TArray<UEdGraphPin*>& PinLinkedTo = CurrentPin->LinkedTo;
					for( int32 J = PinLinkedTo.Num() - 1; J >= 0; --J )
					{
						UEdGraphPin* CurrentLinkedToPin = PinLinkedTo[J];
						if(CurrentLinkedToPin == nullptr)
						{
							PinLinkedTo.RemoveAtSwap(J);
						}
						else
						{
							if(!CurrentLinkedToPin->LinkedTo.Contains(CurrentPin))
							{
								PinLinkedTo.RemoveAtSwap(J);
							}
						}
					}
				}
			}
		}
	}
	CorruptGraphs.Empty();
}

//#define TRACK_PINS

#ifdef TRACK_PINS
TArray<TPair<UEdGraphPin*, FString>> PinAllocationTracking;
#endif //TRACK_PINS

/////////////////////////////////////////////////////
// FEdGraphPinType

void FEdGraphPinType::PostSerialize(const FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_EDGRAPHPINTYPE_SERIALIZATION)
	{
		if (bIsArray_DEPRECATED)
		{
			ContainerType = EPinContainerType::Array;
		}
	}
}

bool FEdGraphPinType::Serialize(FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_EDGRAPHPINTYPE_SERIALIZATION)
	{
		return false;
	}

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::PinsStoreFName)
	{
		Ar << PinCategory;
		Ar << PinSubCategory;
	}
	else
	{
		FString PinCategoryStr;
		Ar << PinCategoryStr;
		PinCategory = *PinCategoryStr;

		FString PinSubCategoryStr;
		Ar << PinSubCategoryStr;
		PinSubCategory = *PinSubCategoryStr;
	}

	if (Ar.UEVer() < VER_UE4_ADDED_SOFT_OBJECT_PATH)
	{
		// Fixup has to be here instead of in BP code because this is embedded in other structures
		if (PinCategory == TEXT("asset"))
		{
			PinCategory = TEXT("softobject");
		}
		else if (PinCategory == TEXT("assetclass"))
		{
			PinCategory = TEXT("softclass");
		}
	}

	// See: FArchive& operator<<( FArchive& Ar, FWeakObjectPtr& WeakObjectPtr )
	// The PinSubCategoryObject should be serialized into the package.
	if(!Ar.IsObjectReferenceCollector() || Ar.IsModifyingWeakAndStrongReferences() || Ar.IsPersistent())
	{
		UObject* Object = PinSubCategoryObject.Get(true);
		Ar << Object;
		if(Ar.IsLoading() || Ar.IsModifyingWeakAndStrongReferences())
		{
			PinSubCategoryObject = Object;
		}
	}

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::EdGraphPinContainerType)
	{
		Ar << ContainerType;
		if (IsMap())
		{
			Ar << PinValueType;
		}
	}
	else
	{
		bool bIsMap = false;
		bool bIsSet = false;
		bool bIsArray = false;

		Ar.UsingCustomVersion(FBlueprintsObjectVersion::GUID);
		if (Ar.CustomVer(FBlueprintsObjectVersion::GUID) >= FBlueprintsObjectVersion::AdvancedContainerSupport)
		{
			Ar << bIsMap;
			if (bIsMap)
			{
				Ar << PinValueType;
			}
			Ar << bIsSet;
		}

		Ar << bIsArray;

		if (Ar.IsLoading())
		{
			ContainerType = ToPinContainerType(bIsArray, bIsSet, bIsMap);
		}
	}

	bool bIsReferenceBool = bIsReference;
	bool bIsWeakPointerBool = bIsWeakPointer;

	Ar << bIsReferenceBool;
	Ar << bIsWeakPointerBool;

	if (Ar.UEVer() >= VER_UE4_MEMBERREFERENCE_IN_PINTYPE)
	{
		Ar << PinSubCategoryMemberReference;
	}
	else if (Ar.IsLoading() && Ar.IsPersistent())
	{
		if ((PinCategory == TEXT("delegate")) || (PinCategory == TEXT("mcdelegate")))
		{
			if (const UFunction* Signature = Cast<const UFunction>(PinSubCategoryObject.Get()))
			{
				PinSubCategoryMemberReference.MemberName = Signature->GetFName();
				PinSubCategoryMemberReference.MemberParent = Signature->GetOwnerClass();
				PinSubCategoryObject = NULL;
			}
		}
	}

	bool bIsConstBool = bIsConst;

	if (Ar.UEVer() >= VER_UE4_SERIALIZE_PINTYPE_CONST)
	{
		Ar << bIsConstBool;
	}

	bool bIsUObjectWrapperBool = bIsUObjectWrapper;

	if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::PinTypeIncludesUObjectWrapperFlag)
	{
		Ar << bIsUObjectWrapperBool;
	}

	FName OldPinCategory = PinCategory;

	if (Ar.IsLoading())
	{
		bIsReference = bIsReferenceBool;
		bIsWeakPointer = bIsWeakPointerBool;
		bIsConst = bIsConstBool;

#if WITH_EDITOR
		// Due to uninitialized memory, bIsUObjectWrapper got serialized as true incorrectly
		// for some number of pins. Set it back to false if it is not an object containing type
		if (bIsUObjectWrapperBool)
		{
			static const TArray<FName> WrappedCategories = 
			{
				TEXT("class"),
				TEXT("object"),
				TEXT("interface"),
				TEXT("softclass"),
				TEXT("softobject")
			};

			if (!WrappedCategories.Contains(PinCategory))
			{
				bIsUObjectWrapperBool = false;
			}
		}

		bool bFixupPinCategories =
			(Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::BlueprintPinsUseRealNumbers) &&
			((PinCategory == TEXT("double")) || (PinCategory == TEXT("float")));

		if (bFixupPinCategories)
		{
			PinCategory = TEXT("real");
			PinSubCategory = TEXT("double");
		}
#endif

		bIsUObjectWrapper = bIsUObjectWrapperBool;
	}

#if WITH_EDITOR
	bool bSerializeAsSinglePrecisionFloatBool = bSerializeAsSinglePrecisionFloat;

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::SerializeFloatPinDefaultValuesAsSinglePrecision)
	{
		Ar << bSerializeAsSinglePrecisionFloatBool;
	}
	else
	{
		if (OldPinCategory == TEXT("float"))
		{
			bSerializeAsSinglePrecisionFloatBool = true;
		}
	}

	bSerializeAsSinglePrecisionFloat = bSerializeAsSinglePrecisionFloatBool;
#endif

	return true;
}

#if WITH_EDITORONLY_DATA
void FEdGraphPinType::DeclareCustomVersions(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
}
#endif

FEdGraphPinType FEdGraphPinType::GetPinTypeForTerminalType( const FEdGraphTerminalType& TerminalType )
{
	FEdGraphPinType TerminalTypeAsPin;
	TerminalTypeAsPin.PinCategory = TerminalType.TerminalCategory;
	TerminalTypeAsPin.PinSubCategory = TerminalType.TerminalSubCategory;
	TerminalTypeAsPin.PinSubCategoryObject = TerminalType.TerminalSubCategoryObject;

	return TerminalTypeAsPin;
}

FEdGraphPinType FEdGraphPinType::GetTerminalTypeForContainer( const FEdGraphPinType& ContainerType )
{
	ensure(ContainerType.IsContainer());
	
	FEdGraphPinType TerminalType = ContainerType;
	TerminalType.ContainerType = EPinContainerType::None;
	return TerminalType;
}

/////////////////////////////////////////////////////
// FEdGraphPinReference
void FEdGraphPinReference::SetPin(const UEdGraphPin* NewPin)
{
	if (NewPin)
	{
		OwningNode = NewPin->GetOwningNodeUnchecked();
		ensure(OwningNode.Get() || NewPin->IsPendingKill());
		PinId = NewPin->PinId;
	}
	else
	{
		OwningNode = nullptr;
		PinId.Invalidate();
	}
}

UEdGraphPin* FEdGraphPinReference::Get() const
{
	UEdGraphNode* ResolvedOwningNode = OwningNode.Get();
	if (ResolvedOwningNode && PinId.IsValid())
	{
		const FGuid& TargetId = PinId;
		UEdGraphPin** ExistingPin = ResolvedOwningNode->Pins.FindByPredicate([TargetId](const UEdGraphPin* TargetPin) { return TargetPin && TargetPin->PinId == TargetId; });
		if (ExistingPin)
		{
			return *ExistingPin;
		}
	}

	return nullptr;
}

/////////////////////////////////////////////////////
// Resolve enums and structs
enum class EPinResolveType : uint8
{
	OwningNode,
	LinkedTo,
	SubPins,
	ParentPin,
	ReferencePassThroughConnection
};

enum class EPinResolveResult : uint8
{
	FailedParse, // failed to even parse the text buffer, meaning we have to bail out
	FailedSafely, // simply failed to resolve the object referenced by the text buffer, meaning we have to disconnect
	Deferred, // owning object was found, but referenced pin is not presenet yet
	Suceeded // immediately resolved
};

struct FUnresolvedPinData
{
	UEdGraphPin* ReferencingPin;
	int32 ArrayIdx;
	EPinResolveType ResolveType;
	bool bResolveSymmetrically;

	FUnresolvedPinData() : ReferencingPin(nullptr), ArrayIdx(INDEX_NONE), ResolveType(EPinResolveType::OwningNode) {}
	FUnresolvedPinData(UEdGraphPin* InReferencingPin, EPinResolveType InResolveType, int32 InArrayIdx, bool bInResolveSymmetrically = false)
		: ReferencingPin(InReferencingPin), ArrayIdx(InArrayIdx), ResolveType(InResolveType), bResolveSymmetrically(bInResolveSymmetrically)
	{}
};

struct FPinResolveId
{
	FGuid PinGuid;
	TWeakObjectPtr<UEdGraphNode> OwningNode;

	FPinResolveId() {}

	FPinResolveId(const FGuid& InPinGuid, const TWeakObjectPtr<UEdGraphNode>& InOwningNode)
		: PinGuid(InPinGuid), OwningNode(InOwningNode)
	{}

	bool operator==(const FPinResolveId& Other) const
	{
		return PinGuid == Other.PinGuid && OwningNode == Other.OwningNode;
	}

	friend inline uint32 GetTypeHash(const FPinResolveId& Id)
	{
		return HashCombine(GetTypeHash(Id.PinGuid), GetTypeHash(Id.OwningNode));
	}
};

/////////////////////////////////////////////////////
// PinHelpers
namespace PinHelpers
{
	static thread_local TMap<FPinResolveId, TArray<FUnresolvedPinData>> UnresolvedPins;
	static thread_local TMap<TWeakObjectPtr<UEdGraphPin_Deprecated>, FGuid> DeprecatedPinToNewPinGUIDMap;
	static thread_local TMap<TWeakObjectPtr<UEdGraphPin_Deprecated>, TArray<FUnresolvedPinData>> UnresolvedDeprecatedPins;

	static const TCHAR ExportTextPropDelimiter = ',';

	static FString PinIdName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, PinId);
	static FString PinNameName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, PinName);
#if WITH_EDITORONLY_DATA
	static FString PinFriendlyNameName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, PinFriendlyName);
#endif
	static FString PinToolTipName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, PinToolTip);
	static FString DirectionName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, Direction);
	static FString PinTypeName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, PinType);
	static FString DefaultValueName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, DefaultValue);
	static FString AutogeneratedDefaultValueName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, AutogeneratedDefaultValue);
	static FString DefaultObjectName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, DefaultObject);
	static FString DefaultTextValueName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, DefaultTextValue);
	static FString LinkedToName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, LinkedTo);
	static FString SubPinsName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, SubPins);
	static FString ParentPinName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, ParentPin);
	static FString ReferencePassThroughConnectionName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, ReferencePassThroughConnection);

#if WITH_EDITORONLY_DATA
	static FString PersistentGuidName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, PersistentGuid);
	static FString bHiddenName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, bHidden);
	static FString bNotConnectableName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, bNotConnectable);
	static FString bDefaultValueIsReadOnlyName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, bDefaultValueIsReadOnly);
	static FString bDefaultValueIsIgnoredName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, bDefaultValueIsIgnored);
	static FString bAdvancedViewName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, bAdvancedView);
	// Don't need bDisplayAsMutableRef's name because it is transient
	static FString bOrphanedPinName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, bOrphanedPin);
	// Don't need bSavePinIfOrphaned's name because it is transient
	// Don't need bWasTrashed's name because it is transient
#endif
	static EPinResolveResult ImportText_PinReference(const TCHAR*& Buffer, UEdGraphPin*& PinRef, int32 ArrayIdx, UEdGraphPin* RequestingPin, EPinResolveType ResolveType);
	static void ResolveReferencesToPin(UEdGraphPin* Pin, TArray<FUnresolvedPinData>& UnresolvedPinData, bool bStrictValidation = true);
}

/////////////////////////////////////////////////////
// UEdGraphPin
UEdGraphPin* UEdGraphPin::CreatePin(UEdGraphNode* InOwningNode)
{
	check(InOwningNode);
	UEdGraphPin* NewPin = new UEdGraphPin(InOwningNode, FGuid::NewGuid());
	return NewPin;
}

void UEdGraphPin::MakeLinkTo(UEdGraphPin* ToPin, bool bAlwaysMarkDirty)
{
	if (ToPin)
	{
		check(!bWasTrashed);

		// Make sure we don't already link to it
		if (!LinkedTo.Contains(ToPin))
		{
			UEdGraphNode* MyNode = GetOwningNode();

			// Check that the other pin does not link to us
			ensureMsgf(!ToPin->LinkedTo.Contains(this), TEXT("%s"), *GetLinkInfoString( LOCTEXT("MakeLinkTo", "MakeLinkTo").ToString(), LOCTEXT("IsLinked", "is linked with pin").ToString(), ToPin));
			ensureMsgf(MyNode->GetOuter() == ToPin->GetOwningNode()->GetOuter(), TEXT("%s"), *GetLinkInfoString( LOCTEXT("MakeLinkTo", "MakeLinkTo").ToString(), LOCTEXT("OuterMismatch", "has a different outer than pin").ToString(), ToPin)); // Ensure both pins belong to the same graph

			// Notify owning nodes about upcoming change
			Modify(bAlwaysMarkDirty);
			ToPin->Modify(bAlwaysMarkDirty);

			// Add to both lists
			LinkedTo.Add(ToPin);
			ToPin->LinkedTo.Add(this);

			// If either node was a pre-placed ghost, turn it into a real thing
			UEdGraphPin::ConvertConnectedGhostNodesToRealNodes(MyNode, bAlwaysMarkDirty);
			UEdGraphPin::ConvertConnectedGhostNodesToRealNodes(ToPin->GetOwningNode(), bAlwaysMarkDirty);
		}
	}
}

#if WITH_EDITORONLY_DATA
void UEdGraphPin::SetSnappedChild(UEdGraphPin* SnappedChild)
{
	check(SnappedChild);
	if( !ensure(SnappedChild->bHasSnappedParent == false && bHasSnappedChild == false) ||
		!ensure(LinkedTo.Contains(SnappedChild)))
	{
		return;
	}

	SnappedChild->bHasSnappedParent = true;
	bHasSnappedChild = true;
}

void UEdGraphPin::ClearSnappedChild()
{
	if(UEdGraphPin* OldSnappedChild = GetSnappedChildPin())
	{
		OldSnappedChild->bHasSnappedParent = false;
	}
	bHasSnappedChild = false;
}

bool UEdGraphPin::IsSnapped() const
{
	return bHasSnappedChild || bHasSnappedParent;
}

bool UEdGraphPin::IsSnappedParentOf(const UEdGraphNode* Node) const
{
	if(!bHasSnappedChild)
	{
		return false;
	}

	if(UEdGraphPin* SnappedChild = GetSnappedChildPin())
	{
		return SnappedChild->GetOwningNode() == Node;
	}

	return false;
}

FSnappedEdGraphPin UEdGraphPin::GetSnapDescription() const
{
	if(!IsSnapped())
	{
		return {};
	}

	// if we have a child attachment return that:
	FSnappedEdGraphPin Result;
	if(bHasSnappedChild)
	{
		if(UEdGraphPin* ChildPin = GetSnappedChildPin())
		{
			Result.ParentPin = const_cast<UEdGraphPin*>(this);
			Result.ChildPin = ChildPin;
		}
	}
	else
	{
		// terminal attachment - fall back to describing our relationship with the parent pin:
		if(UEdGraphPin* SnappedParentPin = GetSnappedParentPin())
		{
			Result.ParentPin = SnappedParentPin;
			Result.ChildPin = const_cast<UEdGraphPin*>(this);
		}
	}
	return Result;
}

UEdGraphNode* UEdGraphPin::GetSnappedChildNode() const
{
	if(UEdGraphPin* SnappedChild = GetSnappedChildPin())
	{
		return SnappedChild->GetOwningNode();
	}
	return nullptr;
}

UEdGraphPin* UEdGraphPin::GetSnappedChildPin() const
{
	if(!bHasSnappedChild)
	{
		return nullptr;
	}
	
	for(UEdGraphPin* LinkedPin : LinkedTo)
	{
		if(LinkedPin->bHasSnappedParent)
		{
			return LinkedPin;
		}
	}
	return nullptr;
}

UEdGraphPin* UEdGraphPin::GetSnappedParentPin() const
{
	if(!bHasSnappedParent)
	{
		return nullptr;
	}

	for(UEdGraphPin* LinkedPin : LinkedTo)
	{
		if(LinkedPin->bHasSnappedChild)
		{
			return LinkedPin;
		}
	}
	return nullptr;
}

#endif

void UEdGraphPin::BreakLinkTo(UEdGraphPin* ToPin, bool bAlwaysMarkDirty)
{
	if (ToPin)
	{
		// If we do indeed link to the passed in pin...
		if (LinkedTo.Contains(ToPin))
		{
			Modify(bAlwaysMarkDirty);

			if (ToPin->LinkedTo.Contains(this))
			{
				ToPin->Modify(bAlwaysMarkDirty);
				ToPin->LinkedTo.Remove(this);
			}
			else if (OwningNode && !OwningNode->HasAnyFlags(RF_BeginDestroyed))
			{
				// Ensure that the other pin links to us but ignore it if our parent is invalid or being destroyed
				ensureAlwaysMsgf(ToPin->LinkedTo.Contains(this), TEXT("%s"), *GetLinkInfoString(LOCTEXT("BreakLinkTo", "BreakLinkTo").ToString(), LOCTEXT("NotLinked", "not reciprocally linked with pin").ToString(), ToPin));
			}
			
#if WITH_EDITORONLY_DATA
			// update snapping state - if any:
			if(bHasSnappedParent && ToPin->bHasSnappedChild)
			{
				ToPin->bHasSnappedChild = false;
				bHasSnappedParent = false;
			}
			else if(bHasSnappedChild && ToPin->bHasSnappedParent)
			{
				bHasSnappedChild = false;
				ToPin->bHasSnappedParent = false;
			}
#endif//WITH_EDITORONLY_DATA

			LinkedTo.Remove(ToPin);
		}
		else
		{
			// Check that the other pin does not link to us
			ensureAlwaysMsgf(!ToPin->LinkedTo.Contains(this), TEXT("%s"), *GetLinkInfoString(LOCTEXT("MakeLinkTo", "MakeLinkTo").ToString(), LOCTEXT("IsLinked", "is linked with pin").ToString(), ToPin));
		}
	}
}

void UEdGraphPin::BreakAllPinLinks(bool bNotifyNodes, bool bAlwaysMarkDirty)
{
	TArray<UEdGraphPin*> LinkedToCopy = LinkedTo;

	for (UEdGraphPin* LinkedToPin : LinkedToCopy)
	{
		BreakLinkTo(LinkedToPin, bAlwaysMarkDirty);
#if WITH_EDITOR
		if (bNotifyNodes)
		{
			if (UEdGraphNode* LinkedToNode = LinkedToPin->GetOwningNodeUnchecked())
			{
				LinkedToNode->PinConnectionListChanged(LinkedToPin);
			}
		}
#endif
	}
}

template<class T>
void UEdGraphPin::TransferPersistentDataFromOldPin(UEdGraphPin& DestPin, T& SourcePin, const UEdGraphPin::ETransferPersistentDataMode TransferMode)
{
	DestPin.Modify(/*bAlwaysMarkDirty =*/ false);

	// The name matches already, doesn't get copied here
	// The PinType, Direction, and bNotConnectable are properties generated from the schema
	DestPin.PinId = SourcePin.PinId;
#if WITH_EDITORONLY_DATA
	DestPin.bHasSnappedChild = SourcePin.bHasSnappedChild;
	DestPin.bHasSnappedParent = SourcePin.bHasSnappedParent;
#endif//WITH_EDITORONLY_DATA

	// Only move the default value if it was modified; inherit the new default value otherwise
	if (DestPin.Direction == EGPD_Input)
	{
		if (!SourcePin.DoesDefaultValueMatchAutogenerated())
		{
			DestPin.DefaultObject = SourcePin.DefaultObject;
			DestPin.DefaultValue = MoveTempIfPossible(SourcePin.DefaultValue);
			DestPin.DefaultTextValue = MoveTempIfPossible(SourcePin.DefaultTextValue);
			DestPin.PinType.bSerializeAsSinglePrecisionFloat = SourcePin.PinType.bSerializeAsSinglePrecisionFloat;
		}
		else
		{
			DestPin.GetSchema()->ResetPinToAutogeneratedDefaultValue(&DestPin, false);
		}
	}

	// Copy the links
	for (UEdGraphPin* OtherPin : SourcePin.LinkedTo)
	{
		check(OtherPin);

		OtherPin->Modify(/*bAlwaysMarkDirty =*/ false);

		DestPin.LinkedTo.Add(OtherPin);

		// Unlike MakeLinkTo(), we attempt to ensure that the new pin (this) is inserted at the same position as the old pin (source)
		// in the OtherPin's LinkedTo array. This is necessary to ensure that the node's position in the execution order will remain
		// unchanged after nodes are reconstructed, because OtherPin may be connected to more than just this node.
		const int32 Index = OtherPin->LinkedTo.IndexOfByKey(&SourcePin);
		if (Index != INDEX_NONE)
		{
			switch (TransferMode)
			{
			case ETransferPersistentDataMode::Move:

				OtherPin->LinkedTo[Index] = &DestPin;
				break;

			case ETransferPersistentDataMode::Copy:

				OtherPin->LinkedTo.Insert(&DestPin, Index);
				break;

			default:
				check(false);
			}
		}
		else
		{
			// Fallback to "normal" add, just in case the old pin doesn't exist in the other pin's LinkedTo array for some reason.
			OtherPin->LinkedTo.Add(&DestPin);
		}
	}

	// If the source pin is split, then split the new one, but don't split multiple times, typically splitting is done
	// by UK2Node::ReallocatePinsDuringReconstruction or FBlueprintEditor::OnSplitStructPin, but there are several code
	// paths into this, and split state should be persistent:
	if (SourcePin.SubPins.Num() > 0 && DestPin.SubPins.Num() == 0)
	{
		DestPin.GetSchema()->SplitPin(&DestPin);
	}

#if WITH_EDITORONLY_DATA
	// Copy advanced visibility property, if it can be changed by user.
	// Otherwise we don't want to copy this, or we'd be ignoring new metadata that tries to hide old pins.
	UEdGraphNode* LocalOwningNode = DestPin.GetOwningNodeUnchecked();
	if (LocalOwningNode != nullptr && LocalOwningNode->CanUserEditPinAdvancedViewFlag())
	{
		DestPin.bAdvancedView = SourcePin.bAdvancedView;
	}
#endif // WITH_EDITORONLY_DATA
}

void UEdGraphPin::MovePersistentDataFromOldPin(UEdGraphPin& SourcePin)
{
	SourcePin.Modify(/*bAlwaysMarkDirty =*/ false);

	TransferPersistentDataFromOldPin(*this, SourcePin, UEdGraphPin::ETransferPersistentDataMode::Move);

	SourcePin.LinkedTo.Reset();
}

void UEdGraphPin::CopyPersistentDataFromOldPin(const UEdGraphPin& SourcePin)
{
	TransferPersistentDataFromOldPin(*this, SourcePin, UEdGraphPin::ETransferPersistentDataMode::Copy);
}

void UEdGraphPin::AssignByRefPassThroughConnection(UEdGraphPin* InTargetPin)
{
	if (InTargetPin)
	{
		if (GetOwningNode() != InTargetPin->GetOwningNode())
		{
			UE_LOGF(LogBlueprint, Warning, "Pin '%ls' is owned by node '%ls' and pin '%ls' is owned by '%ls', they must be owned by the same node!", *GetName(), *GetOwningNode()->GetName(), *InTargetPin->GetName(), *InTargetPin->GetOwningNode()->GetName());
		}
		else if (Direction == InTargetPin->Direction)
		{
			UE_LOGF(LogBlueprint, Warning, "Both pin '%ls' and pin '%ls' on node '%ls' go in the same direction, one must be an input and the other an output!", *GetName(), *InTargetPin->GetName(), *GetOwningNode()->GetName());
		}
		else if (!PinType.bIsReference && !InTargetPin->PinType.bIsReference)
		{
			UEdGraphPin* InputPin = (Direction == EGPD_Input) ? this : InTargetPin;
			UEdGraphPin* OutputPin = (Direction == EGPD_Input) ? InTargetPin : this;
			UE_LOGF(LogBlueprint, Warning, "Input pin '%ls' is attempting to by-ref pass-through to output pin '%ls' on node '%ls', however neither pin is by-ref!", *InputPin->GetName(), *OutputPin->GetName(), *InputPin->GetOwningNode()->GetName());
		}
		else if (!PinType.bIsReference)
		{
			UE_LOGF(LogBlueprint, Warning, "Pin '%ls' on node '%ls' is not by-ref but it is attempting to pass-through to '%ls'", *GetName(), *GetOwningNode()->GetName(), *InTargetPin->GetName());
		}
		else if (!InTargetPin->PinType.bIsReference)
		{
			UE_LOGF(LogBlueprint, Warning, "Pin '%ls' on node '%ls' is not by-ref but it is attempting to pass-through to '%ls'", *InTargetPin->GetName(), *InTargetPin->GetOwningNode()->GetName(), *GetName());
		}
		else
		{
			ReferencePassThroughConnection = InTargetPin;
			InTargetPin->ReferencePassThroughConnection = this;
		}
	}
}

const class UEdGraphSchema* UEdGraphPin::GetSchema() const
{
#if WITH_EDITOR
	UEdGraphNode* OwnerNode = GetOwningNodeUnchecked();
	return OwnerNode ? OwnerNode->GetSchema() : nullptr;
#else
	return nullptr;
#endif	//WITH_EDITOR
}

bool UEdGraphPin::HasAnyConnections() const
{
	bool bHasAnyConnections = false;
	if(SubPins.Num() > 0)
	{
		// we have subpins, defer authority to them:
		for(UEdGraphPin* Pin : SubPins)
		{
			bHasAnyConnections = Pin->HasAnyConnections();
			if(bHasAnyConnections)
			{
				break;
			}
		}
	}
	else
	{
		bHasAnyConnections = LinkedTo.Num() > 0;
	}
	return bHasAnyConnections;
}

FString UEdGraphPin::GetDefaultAsString() const
{
	if(DefaultObject)
	{
		return DefaultObject.GetPathName();
	}
	else if(!DefaultTextValue.IsEmpty())
	{
		FString TextAsString;
		FTextStringHelper::WriteToBuffer(TextAsString, DefaultTextValue);
		return TextAsString;
	}
	else
	{
		return DefaultValue;
	}
}

bool UEdGraphPin::IsDefaultAsStringEmpty() const
{
	return !DefaultObject && DefaultTextValue.IsEmpty() && DefaultValue.IsEmpty();
}

FText UEdGraphPin::GetDefaultAsText() const
{
	if (DefaultObject)
	{
		return FText::AsCultureInvariant(DefaultObject->GetPathName());
	}
	else if (!DefaultTextValue.IsEmpty())
	{
		return DefaultTextValue;
	}
	else if (!DefaultValue.IsEmpty())
	{
		return FText::AsCultureInvariant(DefaultValue);
	}
	return FText::GetEmpty();
}

bool UEdGraphPin::DoesDefaultValueMatchAutogenerated() const
{
	const UEdGraphSchema* Schema = GetSchema();
	if (Schema)
	{
		return Schema->DoesDefaultValueMatchAutogenerated(*this);
	}
	return false;
}

#if WITH_EDITORONLY_DATA
FText UEdGraphPin::GetDisplayName() const
{
	FText DisplayName = FText::GetEmpty();
	const UEdGraphSchema* Schema = GetSchema();
	if (Schema)
	{
		DisplayName = Schema->GetPinDisplayName(this);
	}
	else
	{
		DisplayName = (!PinFriendlyName.IsEmpty()) ? PinFriendlyName : FText::AsCultureInvariant(PinName.ToString());

		bool bShouldUseLocalizedNodeAndPinNames = false;
		GConfig->GetBool( TEXT("Internationalization"), TEXT("ShouldUseLocalizedNodeAndPinNames"), bShouldUseLocalizedNodeAndPinNames, GEditorSettingsIni );
		if (!bShouldUseLocalizedNodeAndPinNames)
		{
			return FText::AsCultureInvariant(DisplayName.BuildSourceString());
		}
	}
	return DisplayName;
}
#endif // WITH_EDITORONLY_DATA

const FString UEdGraphPin::GetLinkInfoString( const FString& InFunctionName, const FString& InInfoData, const UEdGraphPin* InToPin ) const
{
#if WITH_EDITOR
	const FString FromPinName = GetName();
	const UEdGraphNode* FromPinNode = GetOwningNodeUnchecked();
	const FString FromPinNodeName = (FromPinNode != nullptr) ? FromPinNode->GetNodeTitle(ENodeTitleType::ListView).ToString() : TEXT("Unknown");
	
	const FString ToPinName = InToPin->GetName();
	const UEdGraphNode* ToPinNode = InToPin->GetOwningNodeUnchecked();
	const FString ToPinNodeName = (ToPinNode != nullptr) ? ToPinNode->GetNodeTitle(ENodeTitleType::ListView).ToString() : TEXT("Unknown");
	const FString LinkInfo = FString::Printf( TEXT("UEdGraphPin::%s Pin '%s' on node '%s' %s '%s' on node '%s'"), *InFunctionName, *ToPinName, *ToPinNodeName, *InInfoData, *FromPinName, *FromPinNodeName);
	return LinkInfo;
#else
	return FString();
#endif
}

void UEdGraphPin::AddStructReferencedObjects(class FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PinType.PinSubCategoryMemberReference.MemberParent);
	Collector.AddReferencedObject(DefaultObject);
#if WITH_EDITOR
	// EdGraphPins are not serialized via the reflection system, but we need visibility 
	// into pin references for asset reload and potentially reinstancing, so
	// I'm publishing this as an owned reference in editor:
	Collector.AddReferencedObject(PinType.PinSubCategoryObject);
#endif
}

void UEdGraphPin::SerializeAsOwningNode(FArchive& Ar, TArray<UEdGraphPin*>& ArrayRef)
{
	UEdGraphPin::SerializePinArray(Ar, ArrayRef, nullptr, EPinResolveType::OwningNode);
}

bool UEdGraphPin::Modify(bool bAlwaysMarkDirty)
{
	UEdGraphNode* LocalOwningNode = GetOwningNodeUnchecked();
	if (LocalOwningNode)
	{
		return LocalOwningNode->Modify(bAlwaysMarkDirty);
	}

	return false;
}

void UEdGraphPin::SetOwningNode(UEdGraphNode* NewOwningNode)
{
	UEdGraphNode* OldOwningNode = GetOwningNodeUnchecked();
	if (OldOwningNode)
	{
		int32 ExistingPinIdx = OldOwningNode->Pins.IndexOfByKey(this);
		if (ExistingPinIdx != INDEX_NONE)
		{
			OldOwningNode->Pins.RemoveAt(ExistingPinIdx);
		}
	}

	OwningNode = NewOwningNode;

	if (NewOwningNode)
	{
		int32 ExistingPinIdx = NewOwningNode->Pins.IndexOfByPredicate([this](UEdGraphPin* Pin) { return Pin && Pin->PinId == this->PinId; });
		if (ExistingPinIdx == INDEX_NONE)
		{
			NewOwningNode->Pins.Add(this);
		}
		else
		{
			// If a pin with this ID is already in the node, it better be this pin
			check(NewOwningNode->Pins[ExistingPinIdx] == this);
		}
	}
}

UEdGraphPin* UEdGraphPin::CreatePinFromDeprecatedPin(UEdGraphPin_Deprecated* DeprecatedPin)
{
	if (!DeprecatedPin)
	{
		return nullptr;
	}

	UEdGraphNode* DeprecatedPinOwningNode = CastChecked<UEdGraphNode>(DeprecatedPin->GetOuter());

	UEdGraphPin* NewPin = CreatePin(DeprecatedPinOwningNode);
	DeprecatedPinOwningNode->Pins.Add(NewPin);
	NewPin->InitFromDeprecatedPin(DeprecatedPin);

	return NewPin;
}

UEdGraphPin* UEdGraphPin::FindPinCreatedFromDeprecatedPin(UEdGraphPin_Deprecated* DeprecatedPin)
{
	if (!DeprecatedPin)
	{
		return nullptr;
	}

	UEdGraphNode* DeprecatedPinOwningNode = CastChecked<UEdGraphNode>(DeprecatedPin->GetOuter());
	
	UEdGraphPin* ReturnPin = nullptr;
	FGuid PinGuidThatWasCreatedFromDeprecatedPin = PinHelpers::DeprecatedPinToNewPinGUIDMap.FindRef(DeprecatedPin);
	if (PinGuidThatWasCreatedFromDeprecatedPin.IsValid())
	{
		UEdGraphPin** ExistingPin = DeprecatedPinOwningNode->Pins.FindByPredicate([&PinGuidThatWasCreatedFromDeprecatedPin](const UEdGraphPin* Pin) { return Pin && Pin->PinId == PinGuidThatWasCreatedFromDeprecatedPin; });
		if (ExistingPin)
		{
			ReturnPin = *ExistingPin;
		}
	}

	return ReturnPin;
}

bool UEdGraphPin::ExportTextItem(FString& ValueStr, int32 PortFlags) const
{
	const FNameProperty* NamePropCDO = GetDefault<FNameProperty>();
	const FStrProperty* StrPropCDO = GetDefault<FStrProperty>();
	const FTextProperty* TextPropCDO = GetDefault<FTextProperty>();
	const FBoolProperty* BoolPropCDO = GetDefault<FBoolProperty>();
	static UEdGraphPin DefaultPin(nullptr, FGuid());

	ValueStr += "(";

	ValueStr += PinHelpers::PinIdName + TEXT("=");
	PinId.ExportTextItem(ValueStr, FGuid(), nullptr, PortFlags, nullptr);
	ValueStr += PinHelpers::ExportTextPropDelimiter;

	if (PinName != DefaultPin.PinName)
	{
		ValueStr += PinHelpers::PinNameName + TEXT("=");
		NamePropCDO->ExportTextItem_Direct(ValueStr, &PinName, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

#if WITH_EDITORONLY_DATA
	if (!PinFriendlyName.EqualTo(DefaultPin.PinFriendlyName))
	{
		ValueStr += PinHelpers::PinFriendlyNameName + TEXT("=");
		TextPropCDO->ExportTextItem_Direct(ValueStr, &PinFriendlyName, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}
#endif // WITH_EDITORONLY_DATA

	if (PinToolTip != DefaultPin.PinToolTip)
	{
		ValueStr += PinHelpers::PinToolTipName + TEXT("=");
		StrPropCDO->ExportTextItem_Direct(ValueStr, &PinToolTip, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	if (Direction != DefaultPin.Direction)
	{
		const FString DirectionString = UEnum::GetValueAsString(TEXT("/Script/Engine.EEdGraphPinDirection"), Direction);
		ValueStr += PinHelpers::DirectionName + TEXT("=");
		StrPropCDO->ExportTextItem_Direct(ValueStr, &DirectionString, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	for (TFieldIterator<FProperty> FieldIt(FEdGraphPinType::StaticStruct()); FieldIt; ++FieldIt)
	{
		FProperty* Prop = *FieldIt;
		if (Prop->ShouldPort())
		{
			FString PropertyStr;
			const uint8* PropertyAddr = Prop->ContainerPtrToValuePtr<uint8>(&PinType);
			const uint8* DefaultAddr = Prop->ContainerPtrToValuePtr<uint8>(&DefaultPin.PinType);
			Prop->ExportTextItem_Direct(PropertyStr, PropertyAddr, DefaultAddr, NULL, PortFlags, nullptr);

			if (!PropertyStr.IsEmpty())
			{
				ValueStr += PinHelpers::PinTypeName + TEXT(".") + FieldIt->GetName() + "=" + PropertyStr;
				ValueStr += PinHelpers::ExportTextPropDelimiter;
			}
		}
	}

	if (DefaultValue != DefaultPin.DefaultValue)
	{
		ValueStr += PinHelpers::DefaultValueName + TEXT("=");
		StrPropCDO->ExportTextItem_Direct(ValueStr, &DefaultValue, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	if (AutogeneratedDefaultValue != DefaultPin.AutogeneratedDefaultValue)
	{
		ValueStr += PinHelpers::AutogeneratedDefaultValueName + TEXT("=");
		StrPropCDO->ExportTextItem_Direct(ValueStr, &AutogeneratedDefaultValue, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	if (DefaultObject != DefaultPin.DefaultObject)
	{
		const FString DefaultObjectPath = GetPathNameSafe(DefaultObject);
		ValueStr += PinHelpers::DefaultObjectName + TEXT("=");
		StrPropCDO->ExportTextItem_Direct(ValueStr, &DefaultObjectPath, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	if (!DefaultTextValue.EqualTo(DefaultPin.DefaultTextValue))
	{
		ValueStr += PinHelpers::DefaultTextValueName + TEXT("=");
		TextPropCDO->ExportTextItem_Direct(ValueStr, &DefaultTextValue, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	if (LinkedTo.Num() > 0)
	{
		ValueStr += PinHelpers::LinkedToName + TEXT("=") + UEdGraphPin::ExportText_PinArray(LinkedTo);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	if (SubPins.Num() > 0)
	{
		ValueStr += PinHelpers::SubPinsName + TEXT("=") + UEdGraphPin::ExportText_PinArray(SubPins);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	if (ParentPin)
	{
		ValueStr += PinHelpers::ParentPinName + TEXT("=") + UEdGraphPin::ExportText_PinReference(ParentPin);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

	if (ReferencePassThroughConnection)
	{
		ValueStr += PinHelpers::ReferencePassThroughConnectionName + TEXT("=") + UEdGraphPin::ExportText_PinReference(ReferencePassThroughConnection);
		ValueStr += PinHelpers::ExportTextPropDelimiter;
	}

#if WITH_EDITORONLY_DATA
	{
		ValueStr += PinHelpers::PersistentGuidName + TEXT("=");
		PersistentGuid.ExportTextItem(ValueStr, FGuid(), nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;

		ValueStr += PinHelpers::bHiddenName + TEXT("=");
		bool LocalHidden = bHidden;
		BoolPropCDO->ExportTextItem_Direct(ValueStr, &LocalHidden, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;

		ValueStr += PinHelpers::bNotConnectableName + TEXT("=");
		bool LocalNotConnectable = bNotConnectable;
		BoolPropCDO->ExportTextItem_Direct(ValueStr, &LocalNotConnectable, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;

		ValueStr += PinHelpers::bDefaultValueIsReadOnlyName + TEXT("=");
		bool LocalDefaultValueIsReadOnly = bDefaultValueIsReadOnly;
		BoolPropCDO->ExportTextItem_Direct(ValueStr, &LocalDefaultValueIsReadOnly, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;

		ValueStr += PinHelpers::bDefaultValueIsIgnoredName + TEXT("=");
		bool LocalDefaultValueIsIgnored = bDefaultValueIsIgnored;
		BoolPropCDO->ExportTextItem_Direct(ValueStr, &LocalDefaultValueIsIgnored, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;

		ValueStr += PinHelpers::bAdvancedViewName + TEXT("=");
		bool LocalAdvancedView = bAdvancedView;
		BoolPropCDO->ExportTextItem_Direct(ValueStr, &LocalAdvancedView, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;

		// Intentionally not exporting bDisplayAsMutableRef as it is transient

		ValueStr += PinHelpers::bOrphanedPinName + TEXT("=");
		bool LocalOrphanedPin = bOrphanedPin;
		BoolPropCDO->ExportTextItem_Direct(ValueStr, &LocalOrphanedPin, nullptr, nullptr, PortFlags, nullptr);
		ValueStr += PinHelpers::ExportTextPropDelimiter;

		// Intentionally not exporting bSavePinIfOrphaned as it is transient
		// Intentionally not exporting bWasTrashed as it is transient
		// avoid exporting bHasSnappedChild unless it's true, as it defaults to false
		// and older code that processes T3D nodes may be confused by extra properties:
		if(bHasSnappedChild)
		{
			ValueStr += UEdGraphPin::bHasSnappedChildName + TEXT("=");
			bool LocalHasSnappedChildName = bHasSnappedChild;
			BoolPropCDO->ExportTextItem_Direct(ValueStr, &LocalHasSnappedChildName, nullptr, nullptr, PortFlags, nullptr);
			ValueStr += PinHelpers::ExportTextPropDelimiter;
		}
		
		if(bHasSnappedParent)
		{
			ValueStr += UEdGraphPin::bHasSnappedParentName + TEXT("=");
			bool LocalHasSnappedParentName = bHasSnappedParent;
			BoolPropCDO->ExportTextItem_Direct(ValueStr, &LocalHasSnappedParentName, nullptr, nullptr, PortFlags, nullptr);
			ValueStr += PinHelpers::ExportTextPropDelimiter;
		}
	}
#endif //WITH_EDITORONLY_DATA
	ValueStr += ")";

	return true;
}

static void SkipWhitespace(const TCHAR*& Str)
{
	while (Str && FChar::IsWhitespace(*Str))
	{
		Str++;
	}
}

bool UEdGraphPin::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, class UObject* Parent, FOutputDevice* ErrorText)
{
	PinId = FGuid();

	const FNameProperty* NamePropCDO = GetDefault<FNameProperty>();
	const FStrProperty* StrPropCDO = GetDefault<FStrProperty>();
	const FTextProperty* TextPropCDO = GetDefault<FTextProperty>();
	const FBoolProperty* BoolPropCDO = GetDefault<FBoolProperty>();
	UEnum* PinDirectionEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Engine.EEdGraphPinDirection"));

	SkipWhitespace(Buffer);
	if (*Buffer != '(')
	{
		ErrorText->Logf(ELogVerbosity::Warning, TEXT("%s: Missing opening \'(\' while importing property values."), *GetName());

		// Parse error
		return false;
	}

	Buffer++;
	SkipWhitespace(Buffer);

	while (*Buffer != ')')
	{
		const TCHAR* StartBuffer = Buffer;
		if (*Buffer == 0)
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("%s: Unexpected end-of-stream while importing property values."), *GetName());

			// Parse error
			return false;
		}

		while (*Buffer != '=')
		{
			Buffer++;

			if (*Buffer == 0)
			{
				ErrorText->Logf(ELogVerbosity::Warning, TEXT("%s: Unexpected end-of-stream while importing property values."), *GetName());
	
				// Parse error
				return false;
			}
		}

		int32 NumCharsInToken = Buffer - StartBuffer;
		if (NumCharsInToken <= 0)
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("%s: Encountered a missing property token while importing properties."), *GetName());

			// Parse error
			return false;
		}

		// Advance over the '='
		Buffer++;

		FString PropertyToken = FString::ConstructFromPtrSize(StartBuffer, NumCharsInToken);
		PropertyToken.TrimEndInline();
		bool bParseSuccess = false;
		if (PropertyToken == PinHelpers::PinIdName)
		{
			bParseSuccess = PinId.ImportTextItem(Buffer, PortFlags, Parent, ErrorText);
		}
		else if (PropertyToken == PinHelpers::PinNameName)
		{
			Buffer = NamePropCDO->ImportText_Direct(Buffer, &PinName, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
		}
#if WITH_EDITORONLY_DATA
		else if (PropertyToken == PinHelpers::PinFriendlyNameName)
		{
			Buffer = TextPropCDO->ImportText_Direct(Buffer, &PinFriendlyName, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
		}
#endif //WITH_EDITORONLY_DATA
		else if (PropertyToken == PinHelpers::PinToolTipName)
		{
			Buffer = StrPropCDO->ImportText_Direct(Buffer, &PinToolTip, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
		}
		else if (PropertyToken == PinHelpers::DirectionName)
		{
			FString DirectionString;
			Buffer = StrPropCDO->ImportText_Direct(Buffer, &DirectionString, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			if (bParseSuccess)
			{
				int32 EnumIdx = PinDirectionEnum->GetIndexByNameString(DirectionString);
				if (EnumIdx != INDEX_NONE)
				{
					Direction = (EEdGraphPinDirection)PinDirectionEnum->GetValueByIndex(EnumIdx);
				}
				else
				{
					ErrorText->Logf(ELogVerbosity::Warning, TEXT("Unknown pin direction enum value: %s."), *DirectionString);

					// Invalid enum entry
					bParseSuccess = false;
				}
			}
		}
		else if (PropertyToken.StartsWith(PinHelpers::PinTypeName + TEXT(".")))
		{
			FString Right;
			if (ensure(PropertyToken.Split(".", nullptr, &Right, ESearchCase::CaseSensitive)))
			{
				const FName PropertyName(*Right);
				FProperty* FoundProp = FEdGraphPinType::StaticStruct()->FindPropertyByName(PropertyName);
				if (FoundProp)
				{
					uint8* PropertyAddr = FoundProp->ContainerPtrToValuePtr<uint8>(&PinType);
					Buffer = FoundProp->ImportText_Direct(Buffer, PropertyAddr, Parent, PortFlags, ErrorText);
					bParseSuccess = (Buffer != nullptr);
				}
				// UE_DEPRECATED(4.17) - For some time bIsMap and bIsSet would have been in exported text and will cause issues if we don't handle them
				else if (PropertyName == TEXT("bIsMap") || PropertyName == TEXT("bIsSet"))
				{
					bool bDummyBool = false;
					Buffer = BoolPropCDO->ImportText_Direct(Buffer, &bDummyBool, Parent, PortFlags, ErrorText);
					bParseSuccess = (Buffer != nullptr);
				}
				else
				{
					ErrorText->Logf(ELogVerbosity::Warning, TEXT("Unknown pin type member: %s."), *Right);

					// Couldn't find the prop
					bParseSuccess = false;
				}
			}
			else
			{
				ErrorText->Logf(ELogVerbosity::Warning, TEXT("Unable to parse pin type member name from token: %s."), *PropertyToken);

				// Eh, split failed to find the '.' we saw in the token?
				bParseSuccess = false;
			}
		}
		else if (PropertyToken == PinHelpers::DefaultValueName)
		{
			Buffer = StrPropCDO->ImportText_Direct(Buffer, &DefaultValue, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
		}
		else if (PropertyToken == PinHelpers::AutogeneratedDefaultValueName)
		{
			Buffer = StrPropCDO->ImportText_Direct(Buffer, &AutogeneratedDefaultValue, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
		}
		else if (PropertyToken == PinHelpers::DefaultObjectName)
		{
			FString DefaultObjectString;
			Buffer = StrPropCDO->ImportText_Direct(Buffer, &DefaultObjectString, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			if (bParseSuccess)
			{
				DefaultObject = FindObject<UObject>(nullptr, *DefaultObjectString);

#if WITH_EDITORONLY_DATA
				// Fixup redirectors
				while (Cast<UObjectRedirector>(DefaultObject) != nullptr)
				{
					DefaultObject = Cast<UObjectRedirector>(DefaultObject)->DestinationObject;
				}
#endif
			}
		}
		else if (PropertyToken == PinHelpers::DefaultTextValueName)
		{
			Buffer = TextPropCDO->ImportText_Direct(Buffer, &DefaultTextValue, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
		}
		else if (PropertyToken == PinHelpers::LinkedToName)
		{
			bParseSuccess = UEdGraphPin::ImportText_PinArray(Buffer, LinkedTo, this, EPinResolveType::LinkedTo);
		}
		else if (PropertyToken == PinHelpers::SubPinsName)
		{
			bParseSuccess = UEdGraphPin::ImportText_PinArray(Buffer, SubPins, this, EPinResolveType::SubPins);
		}
		else if (PropertyToken == PinHelpers::ParentPinName)
		{
			bParseSuccess = PinHelpers::ImportText_PinReference(Buffer, ParentPin, INDEX_NONE, this, EPinResolveType::ParentPin) != EPinResolveResult::FailedParse;
		}
		else if (PropertyToken == PinHelpers::ReferencePassThroughConnectionName)
		{
			bParseSuccess = PinHelpers::ImportText_PinReference(Buffer, ReferencePassThroughConnection, INDEX_NONE, this, EPinResolveType::ReferencePassThroughConnection) != EPinResolveResult::FailedParse;
		}
#if WITH_EDITORONLY_DATA
		else if (PropertyToken == PinHelpers::PersistentGuidName)
		{
			bParseSuccess = PersistentGuid.ImportTextItem(Buffer, PortFlags, Parent, ErrorText);
		}
		else if (PropertyToken == PinHelpers::bHiddenName)
		{
			bool LocalHidden = bHidden;
			Buffer = BoolPropCDO->ImportText_Direct(Buffer, &LocalHidden, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			bHidden = LocalHidden;
		}
		else if (PropertyToken == PinHelpers::bNotConnectableName)
		{
			bool LocalNotConnectable = bNotConnectable;
			Buffer = BoolPropCDO->ImportText_Direct(Buffer, &LocalNotConnectable, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			bNotConnectable = LocalNotConnectable;
		}
		else if (PropertyToken == PinHelpers::bDefaultValueIsReadOnlyName)
		{
			bool LocalDefaultValueIsReadOnly = bDefaultValueIsReadOnly;
			Buffer = BoolPropCDO->ImportText_Direct(Buffer, &LocalDefaultValueIsReadOnly, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			bDefaultValueIsReadOnly = LocalDefaultValueIsReadOnly;
		}
		else if (PropertyToken == PinHelpers::bDefaultValueIsIgnoredName)
		{
			bool LocalDefaultValueIsIgnored = bDefaultValueIsIgnored;
			Buffer = BoolPropCDO->ImportText_Direct(Buffer, &LocalDefaultValueIsIgnored, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			bDefaultValueIsIgnored = LocalDefaultValueIsIgnored;
		}
		else if (PropertyToken == PinHelpers::bAdvancedViewName)
		{
			bool LocalAdvancedView = bAdvancedView;
			Buffer = BoolPropCDO->ImportText_Direct(Buffer, &LocalAdvancedView, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			bAdvancedView = LocalAdvancedView;
		}
		// Intentionally not importing bDisplayAsMutableRef as it is transient
		else if (PropertyToken == PinHelpers::bOrphanedPinName)
		{
			bool LocalOrphanedPin = bOrphanedPin;
			Buffer = BoolPropCDO->ImportText_Direct(Buffer, &LocalOrphanedPin, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			if (AreOrphanPinsEnabled())
			{
				bOrphanedPin = LocalOrphanedPin;
			}
		}
		// Intentionally not including bSavePinIfOrphaned. It is transient.
		// Intentionally not importing bWasTrashed as it is transient
		else if (PropertyToken == UEdGraphPin::bHasSnappedChildName)
		{
			bool LocalHasSnappedChild = bHasSnappedChild;
			Buffer = BoolPropCDO->ImportText_Direct(Buffer, &LocalHasSnappedChild, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			bHasSnappedChild = LocalHasSnappedChild;
		}
		else if (PropertyToken == UEdGraphPin::bHasSnappedParentName)
		{
			bool LocalHasSnappedParent = bHasSnappedParent;
			Buffer = BoolPropCDO->ImportText_Direct(Buffer, &LocalHasSnappedParent, Parent, PortFlags, ErrorText);
			bParseSuccess = (Buffer != nullptr);
			bHasSnappedParent = LocalHasSnappedParent;
		}
#endif

		if (!bParseSuccess)
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("%s: Parse error while importing property values (PinName = %s)."), *GetName(), *PinName.ToString());

			// Parse error
			return false;
		}

		SkipWhitespace(Buffer);

		if (*Buffer == PinHelpers::ExportTextPropDelimiter)
		{
			Buffer++;
		}

		SkipWhitespace(Buffer);
	}

	// Advance over the last ')'
	Buffer++;

	if (!PinId.IsValid())
	{
		ErrorText->Logf(ELogVerbosity::Warning, TEXT("%s: Invalid PinId after importing property values (PinName = %s)."), *GetName(), *PinName.ToString());

		// Did not contain a valid PinId
		return false;
	}

	// Someone might have been waiting for this pin to be created. Let them know that this pin now exists.
	ResolveReferencesToPin(this);

	return true;
}

FEdGraphTerminalType UEdGraphPin::GetPrimaryTerminalType() const
{
	return FEdGraphTerminalType::FromPinType(PinType);
}

void UEdGraphPin::MarkAsGarbage()
{
	if (!bWasTrashed)
	{
		DestroyImpl(true);
	}
}

void UEdGraphPin::ShutdownVerification()
{
	Purge();
}

void UEdGraphPin::Purge()
{
	FPinDeletionQueue::Get()->Tick(0.f);
}

UEdGraphPin::UEdGraphPin(UEdGraphNode* InOwningNode, const FGuid& PinIdGuid)
	: OwningNode(InOwningNode)
	, PinId(PinIdGuid)
	, PinName()
	, SourceIndex(INDEX_NONE)
	, Direction(EGPD_Input)
#if WITH_EDITORONLY_DATA
	, bHidden(false)
	, bNotConnectable(false)
	, bDefaultValueIsReadOnly(false)
	, bDefaultValueIsIgnored(false)
	, bAdvancedView(false)
	, bDeprecated(false)
	, bDisplayAsMutableRef(false)
	, bAllowFriendlyName(true)
	, bOrphanedPin(false)
	, bSavePinIfOrphaned(true)
	, bUseBackwardsCompatForEmptyAutogeneratedValue(false)
	, bHasSnappedChild(false)
	, bHasSnappedParent(false)
#endif
	, bWasTrashed(false)
#if WITH_EDITORONLY_DATA
	, PinFriendlyName()
#endif // WITH_EDITORONLY_DATA
	, PinToolTip()
	, PinType()
	, DefaultValue()
	, AutogeneratedDefaultValue()
	, DefaultObject(nullptr)
	, DefaultTextValue()
	, LinkedTo()
	, SubPins()
	, ParentPin(nullptr)
	, ReferencePassThroughConnection(nullptr)
#if WITH_EDITORONLY_DATA
	, PersistentGuid()
#endif
{
#ifdef TRACK_PINS
	PinAllocationTracking.Emplace(this, InOwningNode ? InOwningNode->GetName() : FString("UNOWNED"));
#endif //TRACK_PINS
}

UEdGraphPin::~UEdGraphPin()
{
#ifdef TRACK_PINS
	PinAllocationTracking.RemoveAll([this](const TPair<UEdGraphPin*, FString>& Entry) { return Entry.Key == this; });
#endif //TRACK_PINS
}

void UEdGraphPin::ResolveAllPinReferences()
{
	for(TMap<FPinResolveId, TArray<FUnresolvedPinData>>::TIterator Iter(PinHelpers::UnresolvedPins); Iter; ++Iter)
	{
		const FPinResolveId& Key = Iter->Key;
		UEdGraphNode* Node = Key.OwningNode.Get();
		if (!Node)
		{
			continue;
		}

		const FGuid& LocalGuid = Key.PinGuid;
		UEdGraphPin** TargetPin = Node->Pins.FindByPredicate([&LocalGuid](const UEdGraphPin* Pin) { return Pin && Pin->PinId == LocalGuid; });
		if (TargetPin)
		{
			PinHelpers::ResolveReferencesToPin(*TargetPin, Iter->Value);
		}
	}
	PinHelpers::UnresolvedPins.Reset();
}

void UEdGraphPin::InitFromDeprecatedPin(class UEdGraphPin_Deprecated* DeprecatedPin)
{
	check(DeprecatedPin);
	PinHelpers::DeprecatedPinToNewPinGUIDMap.Add(DeprecatedPin, PinId);
	DeprecatedPin->FixupDefaultValue();

	OwningNode = CastChecked<UEdGraphNode>(DeprecatedPin->GetOuter());

	PinName = *DeprecatedPin->PinName;

#if WITH_EDITORONLY_DATA
	PinFriendlyName = DeprecatedPin->PinFriendlyName;
#endif

	PinToolTip = DeprecatedPin->PinToolTip;
	Direction = DeprecatedPin->Direction;
	PinType = DeprecatedPin->PinType;
	DefaultValue = DeprecatedPin->DefaultValue;
	AutogeneratedDefaultValue = DeprecatedPin->AutogeneratedDefaultValue;
	DefaultObject = DeprecatedPin->DefaultObject;
	DefaultTextValue = DeprecatedPin->DefaultTextValue;
	
	LinkedTo.Empty(DeprecatedPin->LinkedTo.Num());
	LinkedTo.SetNum(DeprecatedPin->LinkedTo.Num());
	for (int32 PinIdx = 0; PinIdx < DeprecatedPin->LinkedTo.Num(); ++PinIdx)
	{
		UEdGraphPin_Deprecated* OldLinkedToPin = DeprecatedPin->LinkedTo[PinIdx];
		UEdGraphPin*& NewPin = LinkedTo[PinIdx];
		// check for corrupt data with cross graph links:
		if (OldLinkedToPin && OldLinkedToPin->GetTypedOuter<UEdGraph>() == DeprecatedPin->GetTypedOuter<UEdGraph>())
		{
			UEdGraphPin* ExistingPin = FindPinCreatedFromDeprecatedPin(OldLinkedToPin);
			if (ExistingPin)
			{
				NewPin = ExistingPin;
			}
			else
			{
				TArray<FUnresolvedPinData>& UnresolvedPinData = PinHelpers::UnresolvedDeprecatedPins.FindOrAdd(OldLinkedToPin);
				UnresolvedPinData.Add(FUnresolvedPinData(this, EPinResolveType::LinkedTo, PinIdx));
			}
		}
	}
	
	SubPins.Empty(DeprecatedPin->SubPins.Num());
	SubPins.SetNum(DeprecatedPin->SubPins.Num());
	for (int32 PinIdx = 0; PinIdx < DeprecatedPin->SubPins.Num(); ++PinIdx)
	{
		UEdGraphPin_Deprecated* OldSubPinsPin = DeprecatedPin->SubPins[PinIdx];
		UEdGraphPin*& NewPin = SubPins[PinIdx];
		if (OldSubPinsPin)
		{
			UEdGraphPin* ExistingPin = FindPinCreatedFromDeprecatedPin(OldSubPinsPin);
			if (ExistingPin)
			{
				NewPin = ExistingPin;
			}
			else
			{
				TArray<FUnresolvedPinData>& UnresolvedPinData = PinHelpers::UnresolvedDeprecatedPins.FindOrAdd(OldSubPinsPin);
				UnresolvedPinData.Add(FUnresolvedPinData(this, EPinResolveType::SubPins, PinIdx));
			}
		}
	}
	
	if (DeprecatedPin->ParentPin)
	{
		UEdGraphPin* ExistingPin = FindPinCreatedFromDeprecatedPin(DeprecatedPin->ParentPin);
		if (ExistingPin)
		{
			ParentPin = ExistingPin;
		}
		else
		{
			TArray<FUnresolvedPinData>& UnresolvedPinData = PinHelpers::UnresolvedDeprecatedPins.FindOrAdd(DeprecatedPin->ParentPin);
			UnresolvedPinData.Add(FUnresolvedPinData(this, EPinResolveType::ParentPin, INDEX_NONE));
		}
	}
	
	if (DeprecatedPin->ReferencePassThroughConnection)
	{
		UEdGraphPin* ExistingPin = FindPinCreatedFromDeprecatedPin(DeprecatedPin->ReferencePassThroughConnection);
		if (ExistingPin)
		{
			ReferencePassThroughConnection = ExistingPin;
		}
		else
		{
			TArray<FUnresolvedPinData>& UnresolvedPinData = PinHelpers::UnresolvedDeprecatedPins.FindOrAdd(DeprecatedPin->ReferencePassThroughConnection);
			UnresolvedPinData.Add(FUnresolvedPinData(this, EPinResolveType::ReferencePassThroughConnection, INDEX_NONE));
		}
	}

#if WITH_EDITORONLY_DATA
	bHidden = DeprecatedPin->bHidden;
	bNotConnectable = DeprecatedPin->bNotConnectable;
	bDefaultValueIsReadOnly = DeprecatedPin->bDefaultValueIsReadOnly;
	bDefaultValueIsIgnored = DeprecatedPin->bDefaultValueIsIgnored;
	bAdvancedView = DeprecatedPin->bAdvancedView;
	bDisplayAsMutableRef = DeprecatedPin->bDisplayAsMutableRef;
	bUseBackwardsCompatForEmptyAutogeneratedValue = true; // If it is from deprecated pin format then it is definitely old enough to need this
	bHasSnappedChild = false;
	bHasSnappedParent = false;
	PersistentGuid = DeprecatedPin->PersistentGuid;
#endif // WITH_EDITORONLY_DATA

	// Now resolve any references this pin which was created from a deprecated pin. This should only be from other pins that were created from deprecated pins.
	TArray<FUnresolvedPinData>* UnresolvedPinData = PinHelpers::UnresolvedDeprecatedPins.Find(DeprecatedPin);
	if (UnresolvedPinData)
	{
		for (const FUnresolvedPinData& PinData : *UnresolvedPinData)
		{
			UEdGraphPin* ReferencingPin = PinData.ReferencingPin;
			if (ReferencingPin)
			{
				switch (PinData.ResolveType)
				{
				case EPinResolveType::LinkedTo:
					ReferencingPin->LinkedTo[PinData.ArrayIdx] = this;
					ensure(LinkedTo.Contains(ReferencingPin));
					break;

				case EPinResolveType::SubPins:
					ReferencingPin->SubPins[PinData.ArrayIdx] = this;
					break;

				case EPinResolveType::ParentPin:
					ReferencingPin->ParentPin = this;
					break;

				case EPinResolveType::ReferencePassThroughConnection:
					ReferencingPin->ReferencePassThroughConnection = this;
					break;

				case EPinResolveType::OwningNode:
				default:
					checkf(0, TEXT("Unhandled ResolveType %d"), (int32)PinData.ResolveType);
					break;
				}
			}
		}

		PinHelpers::UnresolvedDeprecatedPins.Remove(DeprecatedPin);
		UnresolvedPinData = nullptr;
	}
}

void UEdGraphPin::DestroyImpl(bool bClearLinks)
{
	if (bClearLinks)
	{
		BreakAllPinLinks();
		if (ParentPin)
		{
			ParentPin->SubPins.Remove(this);
		}
	}
	else
	{
		LinkedTo.Empty();
	}
	OwningNode = nullptr;
	for (int32 SubPinIndex = SubPins.Num() - 1; SubPinIndex >= 0; --SubPinIndex)
	{
		UEdGraphPin* SubPin = SubPins[SubPinIndex];
		if (SubPin && !SubPin->bWasTrashed)
		{
			SubPins[SubPinIndex]->DestroyImpl(bClearLinks);
		}
	}
	SubPins.Reset();
	ParentPin = nullptr;

	// ReferencePassThroughConnection should be symmetrical: the source and target pins should refer to each other.
	// If one pin is destroyed, then we must disconnect the other so that it doesn't inadvertently reference a trashed pin.
	if (ReferencePassThroughConnection != nullptr)
	{
		ensure(ReferencePassThroughConnection->ReferencePassThroughConnection == this);
		ReferencePassThroughConnection->ReferencePassThroughConnection = nullptr;
		ReferencePassThroughConnection = nullptr;
	}

	bWasTrashed = true;
	FPinDeletionQueue::Add(this);
}

bool UEdGraphPin::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	// These properties are in every pin and are unlikely to be removed, so they are native serialized for speed.
	Ar << OwningNode;
	Ar << PinId;

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::PinsStoreFName)
	{
		Ar << PinName;
	}
	else
	{
		FString PinNameStr;
		Ar << PinNameStr;
		PinName = *PinNameStr;
	}

#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		Ar << PinFriendlyName;
	}
#endif

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::EdGraphPinSourceIndex)
	{
		Ar << SourceIndex;
	}

	Ar << PinToolTip;
	Ar << Direction;
	PinType.Serialize(Ar);
	Ar << DefaultValue;
	Ar << AutogeneratedDefaultValue;
	Ar << DefaultObject;
	Ar << DefaultTextValue;

	// There are several options to solve the problem we're fixing here. First,
	// the problem: we have two UObjects (A and B) that own Pin arrays, and within
	// those Pin arrays there are pins with references to pins owned by the other UObject
	// in LinkedTo. We could:
	//  * Use FEdGraphPinReference to 'softly' reference the other pins by outer+name
	//  * Defer resolving of UEdGraphPin* in volatile contexts (e.g. when serialzing)
	//  * Aggressively fix up references:
	// Our current approach is to defer resolving of UEdGraphPin*
	UEdGraphPin::SerializePinArray(Ar, LinkedTo, this, EPinResolveType::LinkedTo);

	// SubPin array is owned by a single UObject, so no complexity of keeping LinkedTo synchronized:
	UEdGraphPin::SerializePinArray(Ar, SubPins, this, EPinResolveType::SubPins);
	TArray<UEdGraphPin*> NoExistingValues; // unused for non-owner paths..
 	SerializePin(Ar, ParentPin, INDEX_NONE, this, EPinResolveType::ParentPin, NoExistingValues);
 	SerializePin(Ar, ReferencePassThroughConnection, INDEX_NONE, this, EPinResolveType::ReferencePassThroughConnection, NoExistingValues);

#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		Ar << PersistentGuid;


		// Do not reorder the persistent properties! 
		// If you are adding a new one, add it to the end, and if you want to remove one, leave the others at the same offset in the uint32
		uint32 BitField = 0;
		BitField |= bHidden << 0;
		BitField |= bNotConnectable << 1;
		BitField |= bDefaultValueIsReadOnly << 2;
		BitField |= bDefaultValueIsIgnored << 3;
		BitField |= bAdvancedView << 4;
		BitField |= bOrphanedPin << 5;
		BitField |= bHasSnappedChild << 6;
		BitField |= bHasSnappedParent << 7;

		const int32 PersistentBits = 8;

		// While these properties are transient from a serialized point of view, they need to be maintained through undo/redo so add them to
		// the bitfield after the persistent bits.
		if (Ar.IsTransacting())
		{
			BitField |= bDisplayAsMutableRef << (PersistentBits + 0);
			BitField |= bSavePinIfOrphaned << (PersistentBits + 1);
			BitField |= bWasTrashed << (PersistentBits + 2);
		}

		Ar << BitField;

		bHidden = !!(BitField & (1 << 0));
		bNotConnectable = !!(BitField & (1 << 1));
		bDefaultValueIsReadOnly = !!(BitField & (1 << 2));
		bDefaultValueIsIgnored = !!(BitField & (1 << 3));
		bAdvancedView = !!(BitField & (1 << 4));
		if (AreOrphanPinsEnabled())
		{
			bOrphanedPin = !!(BitField & (1 << 5));
		}
		bHasSnappedChild = !!(BitField & (1 << 6));
		bHasSnappedParent = !!(BitField & (1 << 7));

		if (Ar.IsTransacting())
		{
			bDisplayAsMutableRef = !!(BitField & (1 << (PersistentBits + 0)));
			bSavePinIfOrphaned = !!(BitField & (1 << (PersistentBits + 1)));
			bWasTrashed = !!(BitField & (1 << (PersistentBits + 2)));
		}

		if (Ar.IsLoading() && Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::ChangeAssetPinsToString)
		{
			bUseBackwardsCompatForEmptyAutogeneratedValue = true;
		}
	}
#endif

	// Strictly speaking we could resolve now when transacting, but
	// we would still have to do the resolve step later to catch
	// any references that are created after we serialize a given node:
	if (Ar.IsLoading()
#if WITH_EDITOR 
		&& !Ar.IsTransacting()
#endif // WITH_EDITOR
		)
	{
		ResolveReferencesToPin(this);
	}

	return true;
}

#if WITH_EDITORONLY_DATA
void UEdGraphPin::DeclarePinCustomVersions(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	FEdGraphPinType::DeclareCustomVersions(Ar);
}
#endif

void UEdGraphPin::ConvertConnectedGhostNodesToRealNodes(UEdGraphNode* InNode, bool bAlwaysMarkDirty)
{
	if (InNode && InNode->IsAutomaticallyPlacedGhostNode())
	{
		// Enable the node and clear the comment
		InNode->Modify(bAlwaysMarkDirty);
		InNode->SetEnabledState(ENodeEnabledState::Enabled, /*bUserAction=*/ false);
		InNode->NodeComment.Empty();
#if WITH_EDITORONLY_DATA
		InNode->bCommentBubbleVisible = false;
#endif

		// Go through all pin connections and enable the nodes. Enabled nodes will prevent further iteration
		for (UEdGraphPin* Pin : InNode->Pins)
		{
			for (UEdGraphPin* OtherPin : Pin->LinkedTo)
			{
				ConvertConnectedGhostNodesToRealNodes(OtherPin->GetOwningNode(), bAlwaysMarkDirty);
			}
		}
	}
}

void UEdGraphPin::ResolveReferencesToPin(UEdGraphPin* Pin, bool bStrictValidation)
{
	check(!Pin->bWasTrashed);
	FPinResolveId ResolveId(Pin->PinId, Pin->OwningNode);
	TArray<FUnresolvedPinData>* UnresolvedPinData = PinHelpers::UnresolvedPins.Find(ResolveId);
	if (UnresolvedPinData)
	{
		PinHelpers::ResolveReferencesToPin(Pin, *UnresolvedPinData, bStrictValidation);
		PinHelpers::UnresolvedPins.Remove(ResolveId);
	}
}

void PinHelpers::ResolveReferencesToPin(UEdGraphPin* Pin, TArray<FUnresolvedPinData>& UnresolvedPinData, bool bStrictValidation)
{
	for (const FUnresolvedPinData& PinData : UnresolvedPinData)
	{
		UEdGraphPin* ReferencingPin = PinData.ReferencingPin;
		switch (PinData.ResolveType)
		{
		case EPinResolveType::LinkedTo:
			if (ensure(ReferencingPin->LinkedTo.Num() > PinData.ArrayIdx))
			{
				ReferencingPin->LinkedTo[PinData.ArrayIdx] = Pin;
			}
			if (PinData.bResolveSymmetrically)
			{
				if (ensure(!Pin->LinkedTo.Contains(ReferencingPin)))
				{
					Pin->LinkedTo.Add(ReferencingPin);
				}
			}
			if (bStrictValidation)
			{
#if WITH_EDITOR
				// When in the middle of a transaction the LinkedTo lists will be in an 
				// indeterminate state. After PostEditUndo runs we could validate LinkedTo
				// coherence.
				ensureAlways(Pin->LinkedTo.Contains(ReferencingPin) || GEditor->Trans->IsObjectTransacting(Pin->GetOwningNode()));
#else
				ensureAlways(Pin->LinkedTo.Contains(ReferencingPin));
#endif//WITH_EDITOR
			}
			break;

		case EPinResolveType::SubPins:
			ReferencingPin->SubPins[PinData.ArrayIdx] = Pin;
			break;

		case EPinResolveType::ParentPin:
			ReferencingPin->ParentPin = Pin;
			break;

		case EPinResolveType::ReferencePassThroughConnection:
			ReferencingPin->ReferencePassThroughConnection = Pin;
			break;

		case EPinResolveType::OwningNode:
		default:
			checkf(0, TEXT("Unhandled ResolveType %d"), (int32)PinData.ResolveType);
			break;
		}
	}
}

void UEdGraphPin::SerializePinArray(FArchive& Ar, TArray<UEdGraphPin*>& ArrayRef, UEdGraphPin* RequestingPin, EPinResolveType ResolveType)
{
	const int32 NumCurrentEntries = ArrayRef.Num();
	int32 ArrayNum = NumCurrentEntries;
	Ar << ArrayNum;
	
	TArray<UEdGraphPin*> OldPins = Ar.IsLoading() && ResolveType == EPinResolveType::OwningNode ? ArrayRef : TArray<UEdGraphPin*>();

	if (Ar.IsLoading())
	{
		if(Ar.IsTransacting() && ResolveType == EPinResolveType::LinkedTo)
		{
			if(ArrayNum < ArrayRef.Num())
			{
				for(int32 I = ArrayRef.Num() - 1; I >=0 ; --I)
				{
					if( !ArrayRef[I]->bWasTrashed &&
#if WITH_EDITOR
						!GEditor->Trans->IsObjectTransacting(ArrayRef[I]->GetOwningNode()) &&
#endif
						ArrayRef[I]->LinkedTo.Contains(RequestingPin) )
					{
						ArrayRef[I]->LinkedTo.Remove(RequestingPin);
					}
				}
			}
		}

		ArrayRef.SetNum(ArrayNum);
	}

	for (int32 PinIdx = 0; PinIdx < ArrayNum; ++PinIdx)
	{
		UEdGraphPin*& PinRef = ArrayRef[PinIdx];
		SerializePin(Ar, PinRef, PinIdx, RequestingPin, ResolveType, OldPins);
	}

	// free unused pins, this only happens when loading and we have already allocated pin data, which currently
	// means we're serializing for undo/redo:
	for (UEdGraphPin* Pin : OldPins)
	{
		if (!Pin->WasTrashed())
		{
#if WITH_EDITOR
			// More complexity to handle asymmetry in the transaction buffer. If our peer node is not in the transaction
			// then we need to take ownership of the entire connection and clear both LinkedTo Arrays:
			for (int32 I = 0; I < Pin->LinkedTo.Num(); ++I)
			{
				UEdGraphPin* Peer = Pin->LinkedTo[I];
				if(ensure(Peer))
				{
					// PeerNode will be null if the pin we were linked to was already thrown away:
					UEdGraphNode* PeerNode = Peer->GetOwningNodeUnchecked();
					if (PeerNode && (!GEditor || !GEditor->Trans->IsObjectTransacting(PeerNode)))
					{
						Pin->BreakLinkTo(Peer);
						--I;
					}
				}
			}
			// We also must clear the Pins subpin array as that pin may have been reused if this one wasn't
			// If it wasn't reused then it'll get destroyed by being in OldPins as well
			Pin->SubPins.Reset();
#endif// WITH_EDITOR
			Pin->DestroyImpl(false);
		}
	}
}

bool UEdGraphPin::SerializePin(FArchive& Ar, UEdGraphPin*& PinRef, int32 ArrayIdx, UEdGraphPin* RequestingPin, EPinResolveType ResolveType, TArray<UEdGraphPin*>& OldPins)
{
	bool bRetVal = true;

	bool bNullPtr = PinRef == nullptr;
	Ar << bNullPtr;
	if (bNullPtr)
	{
		if (Ar.IsLoading())
		{
			PinRef = nullptr;
		}
	}
	else
	{
		UEdGraphNode* LocalOwningNode = nullptr;
		FGuid PinGuid;
		if (!Ar.IsLoading())
		{
#if DO_CHECK
			if(PinRef->bWasTrashed)
			{
				if(RequestingPin)
				{
					if(!RequestingPin->bWasTrashed)
					{
						checkf(false, TEXT("Pin named %s was serialized while trashed - requesting pin named %s owned by node %s"), *(PinRef->PinName.ToString()), *RequestingPin->PinName.ToString(), *RequestingPin->GetOwningNode()->GetName());
					}
					else
					{
						checkf(false, TEXT("Pin named %s was serialized while trashed - requesting pin named %s"), *(PinRef->PinName.ToString()), *RequestingPin->PinName.ToString());
					}
				}
				else
				{
					checkf(false, TEXT("Pin named %s was serialized while trashed with no requesting pin"), *(PinRef->PinName.ToString()));
				}
			}
#endif //DO_CHECK
			LocalOwningNode = PinRef->OwningNode;
			PinGuid = PinRef->PinId;
		}

		Ar << LocalOwningNode;
		Ar << PinGuid;

		// The connected Pin may no longer exist if the node it belonged to failed to load, 
		// treat it as if it was serialized as bNullPtr = true
		if (ResolveType == EPinResolveType::LinkedTo && Ar.IsLoading())
		{
			if (LocalOwningNode == nullptr)
			{
				PinRef = nullptr;
			}
		}
		else
		{
			check(LocalOwningNode);
		}
		check(PinGuid.IsValid());

		if (ResolveType != EPinResolveType::OwningNode)
		{
			if (LocalOwningNode && Ar.IsLoading())
			{
				UEdGraphPin** ExistingPin = LocalOwningNode->Pins.FindByPredicate([&PinGuid](const UEdGraphPin* Pin) { return Pin && Pin->PinId == PinGuid; });
				if (ExistingPin
#if WITH_EDITOR
					// When transacting we're in a volatile state, and we cannot trust any pins we might find:
					&& !Ar.IsTransacting()
#endif//WITH_EDITOR
					)
				{
					PinRef = *ExistingPin;
					check(!PinRef->bWasTrashed);
				}
				else
				{
					check(RequestingPin);
#if WITH_EDITOR
					if(Ar.IsTransacting() && !GEditor->Trans->IsObjectTransacting(LocalOwningNode))
					{
						/* 
							Three possibilities:

							1. This transaction did not alter this node's LinkedTo list, and also did not
								alter ExistingPin's
							2. This transaction has altered this node's LinkedTo list, but did not alter
								ExistingPin's
							3. Mutations to connections have been made outside the transaction system, corrupting
								the transaction buffer

							In case 2 we need to make sure that the LinkedTo connection is symmetrical, but
							in the case of 1 we have to assume that there is already a connection in ExistingPin.
							If ExistingPin's LinkedTo list *was* mutated IsObjectTransacting must return true
							for the owning node.
						*/

						check(ResolveType == EPinResolveType::LinkedTo);

						if(ExistingPin && *ExistingPin)
						{
							FGuid RequestingPinId = RequestingPin->PinId;
							UEdGraphPin** LinkedTo = (*ExistingPin)->LinkedTo.FindByPredicate([&RequestingPinId, RequestingPin](const UEdGraphPin* Pin) { return Pin == RequestingPin && Pin->PinId == RequestingPinId; });
							if (LinkedTo)
							{
								// case 1:
								PinRef = *ExistingPin;
								ensureAlways(!(*ExistingPin)->LinkedTo.Contains(PinRef) || GEditor->Trans->IsObjectTransacting((*ExistingPin)->GetOwningNode()));
							}
							else
							{
								// case 2:
								TArray<FUnresolvedPinData>& UnresolvedPinData = PinHelpers::UnresolvedPins.FindOrAdd(FPinResolveId(PinGuid, LocalOwningNode));
								UnresolvedPinData.Add(FUnresolvedPinData(RequestingPin, ResolveType, ArrayIdx, true));
								bRetVal = false;
							}
						}
						else
						{
							// case 3:
							UE_LOGF(LogBlueprint, Warning, "Pin link in transaction buffer is corrupt - this occurs because node mutations are not recorded in the transaction buffer. Node links will be lost");
							PinRef = nullptr;
							bRetVal = false;

							// After the transaction is complete, we'll want to fix any asymmetries in 
							// pin links, this will be somewhat costly, but a small hitch is much
							// better than crashing:
							GraphConnectionSanitizer.AddCorruptGraph(LocalOwningNode->GetGraph());
						}
					}
					else
#endif // WITH_EDITOR
					{
						TArray<FUnresolvedPinData>& UnresolvedPinData = PinHelpers::UnresolvedPins.FindOrAdd(FPinResolveId(PinGuid, LocalOwningNode));
						UnresolvedPinData.Add(FUnresolvedPinData(RequestingPin, ResolveType, ArrayIdx));
						bRetVal = false;
					}
				}
			}
		}
		else
		{
			if (Ar.IsLoading())
			{
				UEdGraphPin** PinToReuse = OldPins.FindByPredicate([&PinGuid](const UEdGraphPin* Pin) { return Pin && Pin->PinId == PinGuid; });
				if (PinToReuse)
				{
					PinRef = *PinToReuse;
					check(!PinRef->bWasTrashed);
					OldPins.RemoveAtSwap(PinToReuse - OldPins.GetData());
				}
				else
				{
					PinRef = UEdGraphPin::CreatePin(LocalOwningNode);
				}
			}

			PinRef->Serialize(Ar);
			check(!PinRef->bWasTrashed);
		}
	}

	return bRetVal;
}

FString UEdGraphPin::ExportText_PinReference(const UEdGraphPin* Pin)
{
	if (Pin)
	{
		const FString OwningNodeString = Pin->GetOwningNodeUnchecked() ? Pin->OwningNode->GetName() : TEXT("null");
		return OwningNodeString + " " + Pin->PinId.ToString();
	}

	return FString();
}

FString UEdGraphPin::ExportText_PinArray(const TArray<UEdGraphPin*>& PinArray)
{
	FString RetVal;

	RetVal += "(";
	if (PinArray.Num() > 0)
	{
		for (UEdGraphPin* Pin : PinArray)
		{
			RetVal += ExportText_PinReference(Pin);
			RetVal += PinHelpers::ExportTextPropDelimiter;
		}
	}
	RetVal += ")";

	return RetVal;
}

EPinResolveResult PinHelpers::ImportText_PinReference(const TCHAR*& Buffer, UEdGraphPin*& PinRef, int32 ArrayIdx, UEdGraphPin* RequestingPin, EPinResolveType ResolveType)
{
	const TCHAR* BufferStart = Buffer;
	while (*Buffer != PinHelpers::ExportTextPropDelimiter && *Buffer != ')')
	{
		if (*Buffer == 0)
		{
			// Parse error
			return EPinResolveResult::FailedParse;
		}

		Buffer++;
	}

	FString PinRefLine = FString::ConstructFromPtrSize(BufferStart, Buffer - BufferStart);
	FString OwningNodeString;
	FString PinGuidString;
	if (PinRefLine.Split(TEXT(" "), &OwningNodeString, &PinGuidString, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
	{
		FGuid PinGuid;
		if (FGuid::Parse(PinGuidString, PinGuid))
		{
			UEdGraphNode* LocalOwningNode = FindObject<UEdGraphNode>(RequestingPin->GetOwningNode()->GetOuter(), *OwningNodeString);
			if (LocalOwningNode)
			{
				UEdGraphPin** ExistingPin = LocalOwningNode->Pins.FindByPredicate([&PinGuid](const UEdGraphPin* Pin) { return Pin && Pin->PinId == PinGuid; });
				if (ExistingPin)
				{
					PinRef = *ExistingPin;
					return EPinResolveResult::Suceeded;
				}
				else
				{
					TArray<FUnresolvedPinData>& UnresolvedPinData = PinHelpers::UnresolvedPins.FindOrAdd(FPinResolveId(PinGuid, LocalOwningNode));
					UnresolvedPinData.Add(FUnresolvedPinData(RequestingPin, ResolveType, ArrayIdx));
					return EPinResolveResult::Deferred;
				}
			}
		}
		
	}

	return EPinResolveResult::FailedSafely;
}

bool UEdGraphPin::ImportText_PinArray(const TCHAR*& Buffer, TArray<UEdGraphPin*>& ArrayRef, UEdGraphPin* RequestingPin, EPinResolveType ResolveType)
{
	bool bSuccess = false;

	SkipWhitespace(Buffer);
	if (*Buffer == '(')
	{
		Buffer++;
		SkipWhitespace(Buffer);
		bool bParseError = false;
		while (*Buffer != ')')
		{
			if (*Buffer == 0)
			{
				return false;
			}

			int32 NewItemIdx = ArrayRef.Add(nullptr);
			EPinResolveResult ResolveResult = PinHelpers::ImportText_PinReference(Buffer, ArrayRef[NewItemIdx], NewItemIdx, RequestingPin, ResolveType);

			if (ResolveResult == EPinResolveResult::FailedParse)
			{
				ArrayRef.Pop();
				return false;
			}
			else if (ResolveResult == EPinResolveResult::FailedSafely)
			{
				ArrayRef.Pop();
			}

			if (*Buffer == PinHelpers::ExportTextPropDelimiter)
			{
				// Advance over the ','
				Buffer++;
			}

			SkipWhitespace(Buffer);
		}

		// Advance over the last ')'
		Buffer++;

		return true;
	}

	return false;
}

#if WITH_EDITORONLY_DATA
FString UEdGraphPin::bHasSnappedChildName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, bHasSnappedChild);
FString UEdGraphPin::bHasSnappedParentName = GET_MEMBER_NAME_STRING_CHECKED(UEdGraphPin, bHasSnappedParent);
#endif

int32 GCVarDisableOrphanPins = 0;
FAutoConsoleVariableRef CVarDisableOrphanPins(TEXT("DisableOrphanPins"), GCVarDisableOrphanPins, TEXT("0=Orphan pins are enabled (default), 1=Orphan pins are disabled (note: this option will go away in the future)"), ECVF_ReadOnly);

bool UEdGraphPin::AreOrphanPinsEnabled()
{
	return (GCVarDisableOrphanPins == 0);
}

void UEdGraphPin::SanitizePinsPostUndoRedo()
{
	ensure(PinHelpers::UnresolvedPins.Num() == 0);
	GraphConnectionSanitizer.SanitizeConnectionsForCorruptGraphs();
}

#if WITH_EDITORONLY_DATA
void UEdGraphPin::SetSavePinIfOrphaned(const bool bShouldSave)
{
	if (bSavePinIfOrphaned != bShouldSave)
	{
		bSavePinIfOrphaned = bShouldSave;
		// If we are not saving this pin, we cannot save any children either
		if (!bSavePinIfOrphaned)
		{
			for (UEdGraphPin* SubPin : SubPins)
			{
				SubPin->SetSavePinIfOrphaned(false);
			}
		}
	}
}
#endif

/////////////////////////////////////////////////////
// UEdGraphPin

UEdGraphPin_Deprecated::UEdGraphPin_Deprecated(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bHidden = false;
	bNotConnectable = false;
	bAdvancedView = false;
#endif // WITH_EDITORONLY_DATA
}

void UEdGraphPin_Deprecated::FixupDefaultValue()
{
	static FName GameplayTagName = "GameplayTag";
	static FName GameplayTagContainerName = "GameplayTagContainer";
	static FName GameplayTagsPathName = "/Script/GameplayTags";

	if (PinType.PinSubCategoryObject.IsValid())
	{
		UObject* PinSubCategoryObject = PinType.PinSubCategoryObject.Get();
		if (PinSubCategoryObject->GetOuter()->GetFName() == GameplayTagsPathName)
		{
			if (PinSubCategoryObject->GetFName() == GameplayTagName)
			{
				// Pins of type FGameplayTag were storing "()" for empty arrays and then importing that into ArrayProperty and expecting an empty array.
				// That it was working was a bug and has been fixed, so let's fixup pins. A pin that wants an array size of 1 will always fill the parenthesis
				// so there is no worry about breaking those cases.
				if (DefaultValue == TEXT("()"))
				{
					DefaultValue.Empty();
				}
			}
			else if (PinSubCategoryObject->GetFName() == GameplayTagContainerName)
			{
				// Pins of type FGameplayTagContainer were storing "GameplayTags=()" for empty arrays, which equates to having a single item, default generated as detailed above for FGameplayTag.
				// The solution is to replace occurances with an empty string, due to the item being a struct, we can't just empty the value and must replace only the section we need.
				DefaultValue.ReplaceInline(TEXT("GameplayTags=()"), TEXT("GameplayTags="));
			}
		}
	}
}

FString FGraphDisplayInfo::GetNotesAsString() const
{
	FString Result;

	if (Notes.Num() > 0)
	{
		Result = TEXT("(");
		for (int32 NoteIndex = 0; NoteIndex < Notes.Num(); ++NoteIndex)
		{
			if (NoteIndex > 0)
			{
				Result += TEXT(", ");
			}
			Result += Notes[NoteIndex];
		}
		Result += TEXT(")");
	}

	return Result;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
