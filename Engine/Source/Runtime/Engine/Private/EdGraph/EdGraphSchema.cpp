// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/EdGraphSchema.h"
#include "HAL/IConsoleManager.h"
#include "UObject/MetaData.h"
#include "UObject/TextProperty.h"
#include "EdGraph/EdGraph.h"
#if WITH_EDITOR
#include "Kismet2/Kismet2NameValidators.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectIterator.h"
#include "ScopedTransaction.h"
#include "EditorCategoryUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "String/ParseTokens.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema)

#define LOCTEXT_NAMESPACE "EdGraph"

FReply FEdGraphSchemaAction::OnDoubleClick(UBlueprint* InBlueprint)
{
	return FReply::Unhandled();
}

void FEdGraphSchemaAction::CosmeticUpdateCategory(FText NewCategory)
{
	Category = MoveTemp(NewCategory);

	Category.ToString().ParseIntoArray(LocalizedFullSearchCategoryArray, TEXT(" "), true);
	Category.BuildSourceString().ParseIntoArray(FullSearchCategoryArray, TEXT(" "), true);

	// Glob search text together, we use the SearchText string for basic filtering:
#if WITH_EDITORONLY_DATA
	CategoryChain.Reset();
#endif // WITH_EDITORONLY_DATA
	UpdateSearchText();
}

#if WITH_EDITOR
void FEdGraphSchemaAction::CosmeticUpdateRootCategory(FText NewRootCategory)
{
	if (!NewRootCategory.IsEmpty())
	{
		if (GetCategory().IsEmpty())
		{
			CosmeticUpdateCategory(NewRootCategory);
		}
		else
		{
			CosmeticUpdateCategory(FText::FromString(FString::Format(TEXT("{0}|{1}"), { NewRootCategory.ToString(), GetCategory().ToString()})));
		}
	}
}

const TArray<FString>& FEdGraphSchemaAction::GetCategoryChain() const
{
	return CategoryChain;
}
#endif // WITH_EDITOR

void FEdGraphSchemaAction::UpdateSearchText()
{
	SearchText.Reset();

	for (FString& Entry : LocalizedFullSearchTitlesArray)
	{
		Entry.ToLowerInline();
		SearchText += Entry;
	}

	SearchText.Append(LINE_TERMINATOR);

	for (FString& Entry : LocalizedFullSearchKeywordsArray)
	{
		Entry.ToLowerInline();
		SearchText += Entry;
	}

	SearchText.Append(LINE_TERMINATOR);

	for (FString& Entry : LocalizedFullSearchCategoryArray)
	{
		Entry.ToLowerInline();
		SearchText += Entry;
	}

	SearchText.Append(LINE_TERMINATOR);

	for (FString& Entry : FullSearchTitlesArray)
	{
		Entry.ToLowerInline();
		SearchText += Entry;
	}

	SearchText.Append(LINE_TERMINATOR);

	for (FString& Entry : FullSearchKeywordsArray)
	{
		Entry.ToLowerInline();
		SearchText += Entry;
	}

	SearchText.Append(LINE_TERMINATOR);

	for (FString& Entry : FullSearchCategoryArray)
	{
		Entry.ToLowerInline();
		SearchText += Entry;
	}

	#if WITH_EDITOR
	ensure(CategoryChain.IsEmpty());
	const FStringView CategoryDelim = FStringView(TEXT("|"));
	const UE::String::EParseTokensOptions ParseOptions =
		UE::String::EParseTokensOptions::IgnoreCase |
		UE::String::EParseTokensOptions::SkipEmpty |
		UE::String::EParseTokensOptions::Trim;
	FString SubCategory = FEditorCategoryUtils::GetCategoryDisplayString(GetCategory().ToString());
	UE::String::ParseTokens(FStringView(SubCategory), CategoryDelim,
		[&CategoryChain = CategoryChain](FStringView Token) { CategoryChain.Emplace(Token); },
		ParseOptions);
	#endif
}

void FEdGraphSchemaAction::UpdateSearchData(FText NewMenuDescription, FText NewToolTipDescription, FText NewCategory, FText NewKeywords)
{
	MenuDescription = MoveTemp(NewMenuDescription);
	TooltipDescription = MoveTemp(NewToolTipDescription);
	Category = MoveTemp(NewCategory);
	Keywords = MoveTemp(NewKeywords);

	MenuDescription.ToString().ParseIntoArray(LocalizedMenuDescriptionArray, TEXT(" "), true);
	MenuDescription.BuildSourceString().ParseIntoArray(MenuDescriptionArray, TEXT(" "), true);

	FullSearchTitlesArray = MenuDescriptionArray;
	LocalizedFullSearchTitlesArray = LocalizedMenuDescriptionArray;

	Keywords.ToString().ParseIntoArray(LocalizedFullSearchKeywordsArray, TEXT(" "), true);
	Keywords.BuildSourceString().ParseIntoArray(FullSearchKeywordsArray, TEXT(" "), true);
	
	Category.ToString().ParseIntoArray(LocalizedFullSearchCategoryArray, TEXT(" "), true);
	Category.BuildSourceString().ParseIntoArray(FullSearchCategoryArray, TEXT(" "), true);

	// Glob search text together, we use the SearchText string for basic filtering:
	UpdateSearchText();
}

/////////////////////////////////////////////////////
// FGraphActionListBuilderBase

void FGraphActionListBuilderBase::AddAction( const TSharedPtr<FEdGraphSchemaAction>& NewAction)
{
	Entries.Add( NewAction );
}

void FGraphActionListBuilderBase::AddAction(const TSharedPtr<FEdGraphSchemaAction>& NewAction, FString const& Category)
{
#if WITH_EDITOR
	NewAction->CosmeticUpdateRootCategory(FText::FromString(Category));
#endif
	Entries.Add(NewAction);
}

void FGraphActionListBuilderBase::AddActionList( const TArray<TSharedPtr<FEdGraphSchemaAction> >& NewActions, FString const& Category)
{
#if WITH_EDITOR
	NewActions[0]->CosmeticUpdateRootCategory(FText::FromString(Category));
#endif
	Entries.Add( NewActions[0] );
}

void FGraphActionListBuilderBase::Append( FGraphActionListBuilderBase& Other )
{
	Entries.Append( MoveTemp(Other.Entries) );
}

int32 FGraphActionListBuilderBase::GetNumActions() const
{
	return Entries.Num();
}

TSharedPtr<FEdGraphSchemaAction>& FGraphActionListBuilderBase::GetSchemaAction( const int32 Index)
{
	return Entries[Index];
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGraphActionListBuilderBase::ActionGroup FGraphActionListBuilderBase::GetAction(const int32 Index)
{
	return FGraphActionListBuilderBase::ActionGroup(Entries[Index]);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FGraphActionListBuilderBase::Empty()
{
	Entries.Empty();
}

/////////////////////////////////////////////////////
// FGraphActionListBuilderBase::GraphAction
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGraphActionListBuilderBase::ActionGroup::ActionGroup(TSharedPtr<FEdGraphSchemaAction> InAction, FString CategoryPrefix)
	: RootCategory(MoveTemp(CategoryPrefix))
{
	Actions.Add( InAction );
	InitCategoryChain();
}

FGraphActionListBuilderBase::ActionGroup::ActionGroup( const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, FString CategoryPrefix)
	: RootCategory(MoveTemp(CategoryPrefix))
{
	Actions = InActions;
	InitCategoryChain();
}

FGraphActionListBuilderBase::ActionGroup::ActionGroup(FGraphActionListBuilderBase::ActionGroup && Other)
{
	Move(Other);
}

FGraphActionListBuilderBase::ActionGroup& FGraphActionListBuilderBase::ActionGroup::operator=(FGraphActionListBuilderBase::ActionGroup && Other)
{
	if (&Other != this)
	{
		Move(Other);
	}
	return *this;
}

FGraphActionListBuilderBase::ActionGroup::ActionGroup(const FGraphActionListBuilderBase::ActionGroup& Other)
{
	Copy(Other);
}

FGraphActionListBuilderBase::ActionGroup& FGraphActionListBuilderBase::ActionGroup::operator=(const FGraphActionListBuilderBase::ActionGroup& Other)
{
	if (&Other != this)
	{
		Copy(Other);
	}
	return *this;
}

FGraphActionListBuilderBase::ActionGroup::~ActionGroup()
{
}

const TArray<FString>& FGraphActionListBuilderBase::ActionGroup::GetCategoryChain() const
{
#if WITH_EDITOR
	return CategoryChain;
#else
	static TArray<FString> Dummy;
	return Dummy;
#endif
}

void FGraphActionListBuilderBase::ActionGroup::PerformAction( class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const UE::Slate::FDeprecateVector2DParameter& Location )
{
	for ( int32 ActionIndex = 0; ActionIndex < Actions.Num(); ActionIndex++ )
	{
		TSharedPtr<FEdGraphSchemaAction> CurrentAction = Actions[ActionIndex];
		if ( CurrentAction.IsValid() )
		{
			CurrentAction->PerformAction( ParentGraph, FromPins, Location );
		}
	}
}

void FGraphActionListBuilderBase::ActionGroup::Move(FGraphActionListBuilderBase::ActionGroup& Other)
{
	Actions = MoveTemp(Other.Actions);
	RootCategory = MoveTemp(Other.RootCategory);
	CategoryChain = MoveTemp(Other.CategoryChain);
}

void FGraphActionListBuilderBase::ActionGroup::Copy(const ActionGroup& Other)
{
	Actions = Other.Actions;
	RootCategory = Other.RootCategory;
	CategoryChain = Other.CategoryChain;
}

void FGraphActionListBuilderBase::ActionGroup::InitCategoryChain()
{
#if WITH_EDITOR
	const FStringView CategoryDelim = FStringView(TEXT("|"));
	const UE::String::EParseTokensOptions ParseOptions = 
		UE::String::EParseTokensOptions::IgnoreCase |
		UE::String::EParseTokensOptions::SkipEmpty |
		UE::String::EParseTokensOptions::Trim;
	UE::String::ParseTokens(FStringView(FEditorCategoryUtils::GetCategoryDisplayString(RootCategory)), CategoryDelim,
		[&CategoryChain = CategoryChain](FStringView Token) { CategoryChain.Emplace(Token); },
		ParseOptions);

	if (Actions.Num() > 0)
	{
		FString SubCategory = FEditorCategoryUtils::GetCategoryDisplayString(Actions[0]->GetCategory().ToString());
		UE::String::ParseTokens(FStringView(SubCategory), CategoryDelim,
			[&CategoryChain = CategoryChain](FStringView Token) { CategoryChain.Emplace(Token); },
			ParseOptions);
	}
#endif
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

/////////////////////////////////////////////////////
// FCategorizedGraphActionListBuilder

static FString ConcatCategories(FString RootCategory, FString const& SubCategory)
{
	FString ConcatedCategory = MoveTemp(RootCategory);
	if (!SubCategory.IsEmpty() && !ConcatedCategory.IsEmpty())
	{
		ConcatedCategory += TEXT("|");
	}
	ConcatedCategory += SubCategory;

	return ConcatedCategory;
}

FCategorizedGraphActionListBuilder::FCategorizedGraphActionListBuilder(FString CategoryIn)
	: Category(MoveTemp(CategoryIn))
{
}

void FCategorizedGraphActionListBuilder::AddAction(const TSharedPtr<FEdGraphSchemaAction>& NewAction)
{
	// apply Category to NewAction:
#if WITH_EDITOR
	NewAction->CosmeticUpdateRootCategory(FText::FromString(Category));
#endif
	FGraphActionListBuilderBase::AddAction(NewAction);
}

void FCategorizedGraphActionListBuilder::AddAction(const TSharedPtr<FEdGraphSchemaAction>& NewAction, FString const& CategoryIn)
{
#if WITH_EDITOR
	NewAction->CosmeticUpdateRootCategory(FText::FromString(ConcatCategories(Category, CategoryIn)));
#endif
	FGraphActionListBuilderBase::AddAction(NewAction);
}

void FCategorizedGraphActionListBuilder::AddActionList(TArray<TSharedPtr<FEdGraphSchemaAction> > const& NewActions, FString const& CategoryIn)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGraphActionListBuilderBase::AddActionList(NewActions, ConcatCategories(Category, CategoryIn));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

/////////////////////////////////////////////////////
// FGraphContextMenuBuilder

FGraphContextMenuBuilder::FGraphContextMenuBuilder(const UEdGraph* InGraph) 
	: CurrentGraph(InGraph)
{
	OwnerOfTemporaries =  NewObject<UEdGraph>((UObject*)GetTransientPackage());
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_NewNode

namespace 
{
	// Maximum distance a drag can be off a node edge to require 'push off' from node
	const int32 NodeDistance = 60;
}

UEdGraphNode* FEdGraphSchemaAction_NewNode::CreateNode(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& Location, class UEdGraphNode* InNodeTemplate)
{
	// Duplicate template node to create new node
	UEdGraphNode* ResultNode = nullptr;

#if WITH_EDITOR
	ResultNode = DuplicateObject<UEdGraphNode>(InNodeTemplate, ParentGraph);
	ResultNode->SetFlags(RF_Transactional);

	ParentGraph->AddNode(ResultNode, true);

	ResultNode->CreateNewGuid();
	ResultNode->PostPlacedNewNode();
	ResultNode->AllocateDefaultPins();
	ResultNode->AutowireNewNode(FromPin);

	// For input pins, new node will generally overlap node being dragged off
	// Work out if we want to visually push away from connected node
	int32 XLocation = Location.X;
	if (FromPin && FromPin->Direction == EGPD_Input)
	{
		UEdGraphNode* PinNode = FromPin->GetOwningNode();
		const float XDelta = FMath::Abs(PinNode->NodePosX - Location.X);

		if (XDelta < NodeDistance)
		{
			// Set location to edge of current node minus the max move distance
			// to force node to push off from connect node enough to give selection handle
			XLocation = PinNode->NodePosX - NodeDistance;
		}
	}

	ResultNode->NodePosX = XLocation;
	ResultNode->NodePosY = Location.Y;
	ResultNode->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);
#endif // WITH_EDITOR

	return ResultNode;
}

UEdGraphNode* FEdGraphSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UEdGraphNode* ResultNode = nullptr;

#if WITH_EDITOR
	// If there is a template, we actually use it
	if (NodeTemplate != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));
		ParentGraph->Modify();
		if (FromPin)
		{
			FromPin->Modify();
		}

		ResultNode = CreateNode(ParentGraph, FromPin, Location, NodeTemplate);
	}
#endif // WITH_EDITOR

	return ResultNode;
}

UEdGraphNode* FEdGraphSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/) 
{
	UEdGraphNode* ResultNode = nullptr;

#if WITH_EDITOR
	if (FromPins.Num() > 0)
	{
		ResultNode = PerformAction(ParentGraph, FromPins[0], Location, bSelectNewNode);

		// Try autowiring the rest of the pins
		for (int32 Index = 1; Index < FromPins.Num(); ++Index)
		{
			ResultNode->AutowireNewNode(FromPins[Index]);
		}
	}
	else
	{
		ResultNode = PerformAction(ParentGraph, nullptr, Location, bSelectNewNode);
	}
#endif // WITH_EDITOR

	return ResultNode;
}

void FEdGraphSchemaAction_NewNode::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEdGraphSchemaAction::AddReferencedObjects( Collector );

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject( NodeTemplate );
}

/////////////////////////////////////////////////////
// UEdGraphSchema

UEdGraphSchema::UEdGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UEdGraphSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	const FPinConnectionResponse Response = CanCreateConnection(PinA, PinB);
	bool bModified = false;

	switch (Response.Response)
	{
	case CONNECT_RESPONSE_MAKE:
		PinA->MakeLinkTo(PinB);
		bModified = true;
		break;

	case CONNECT_RESPONSE_BREAK_OTHERS_A:
		PinA->BreakAllPinLinks(true);
		PinA->MakeLinkTo(PinB);
		bModified = true;
		break;

	case CONNECT_RESPONSE_BREAK_OTHERS_B:
		PinB->BreakAllPinLinks(true);
		PinA->MakeLinkTo(PinB);
		bModified = true;
		break;

	case CONNECT_RESPONSE_BREAK_OTHERS_AB:
		PinA->BreakAllPinLinks(true);
		PinB->BreakAllPinLinks(true);
		PinA->MakeLinkTo(PinB);
		bModified = true;
		break;

	case CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE:
		bModified = CreateAutomaticConversionNodeAndConnections(PinA, PinB);
		break;

	case CONNECT_RESPONSE_MAKE_WITH_PROMOTION:
		bModified = CreatePromotedConnection(PinA, PinB);
		break;

	case CONNECT_RESPONSE_DISALLOW:
	default:
		break;
	}

#if WITH_EDITOR
	if (bModified)
	{
		PinA->GetOwningNode()->PinConnectionListChanged(PinA);
		PinB->GetOwningNode()->PinConnectionListChanged(PinB);
	}
#endif	//#if WITH_EDITOR

	return bModified;
}

bool UEdGraphSchema::IsConnectionRelinkingAllowed(UEdGraphPin* InPin) const
{
	return false;
}

const FPinConnectionResponse UEdGraphSchema::CanRelinkConnectionToPin(const UEdGraphPin* OldSourcePin, const UEdGraphPin* TargetPinCandidate) const
{
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Not implemented by this schema"));
}

bool UEdGraphSchema::TryRelinkConnectionTarget(UEdGraphPin* SourcePin, UEdGraphPin* OldTargetPin, UEdGraphPin* NewTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes) const
{
	return false;
}

bool UEdGraphSchema::CreateAutomaticConversionNodeAndConnections(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	return false;
}

bool UEdGraphSchema::CreatePromotedConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	return false;
}

#if WITH_EDITORONLY_DATA
bool UEdGraphSchema::ShouldShowPanelContextMenuForIncompatibleConnections() const
{
	return ShouldShowPanelContextMenuForIncompatibleConnectionsImpl();
}
#endif

void UEdGraphSchema::TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified) const
{
	Pin.DefaultValue = NewDefaultValue;

#if WITH_EDITOR
	UEdGraphNode* Node = Pin.GetOwningNode();
	check(Node);
	Node->PinDefaultValueChanged(&Pin);
#endif	//#if WITH_EDITOR
}

void UEdGraphSchema::TrySetDefaultObject(UEdGraphPin& Pin, UObject* NewDefaultObject, bool bMarkAsModified) const
{
	Pin.DefaultObject = NewDefaultObject;

#if WITH_EDITOR
	UEdGraphNode* Node = Pin.GetOwningNode();
	check(Node);
	Node->PinDefaultValueChanged(&Pin);
#endif	//#if WITH_EDITOR
}

void UEdGraphSchema::TrySetDefaultText(UEdGraphPin& InPin, const FText& InNewDefaultText, bool bMarkAsModified) const
{
	InPin.DefaultTextValue = InNewDefaultText;

#if WITH_EDITOR
	UEdGraphNode* Node = InPin.GetOwningNode();
	check(Node);
	Node->PinDefaultValueChanged(&InPin);
#endif	//#if WITH_EDITOR
}

bool UEdGraphSchema::DoesDefaultValueMatch(const UEdGraphPin& InPin, const FString& InValue) const
{
	if (InValue.IsEmpty())
	{
		return InPin.IsDefaultAsStringEmpty();
	}

	// Same logic as UEdGraphPin::DoesDefaultValueMatchAutogenerated
	// Ignoring case on purpose to match default behavior
	FText ValueAsText;
	if (!InPin.DefaultTextValue.IsEmpty() && FTextStringHelper::ReadFromBuffer(*InValue, ValueAsText))
	{
		return FTextProperty::Identical_Implementation(ValueAsText, InPin.DefaultTextValue, 0);
	}
	else
	{
		return InPin.GetDefaultAsString().Equals(InValue, ESearchCase::IgnoreCase);
	}
}

bool UEdGraphSchema::DoesDefaultValueMatchAutogenerated(const UEdGraphPin& InPin) const
{
	// Specifically call our version of this function rather than any overrides
	// This maintains the existing logic of this function while sharing the implementation
	return UEdGraphSchema::DoesDefaultValueMatch(InPin, InPin.AutogeneratedDefaultValue);
}

void UEdGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
#if WITH_EDITOR
	TSet<UEdGraphNode*> NodeList;
	NodeList.Add(&TargetNode);
	
	// Iterate over each pin and break all links
	for (TArray<UEdGraphPin*>::TIterator PinIt(TargetNode.Pins); PinIt; ++PinIt)
	{
		UEdGraphPin* TargetPin = *PinIt;
		if (TargetPin != nullptr && TargetPin->SubPins.Num() == 0)
		{
			// Keep track of which node(s) the pin's connected to
			for (UEdGraphPin*& OtherPin : TargetPin->LinkedTo)
			{
				if (OtherPin)
				{
					UEdGraphNode* OtherNode = OtherPin->GetOwningNode();
					if (OtherNode)
					{
						NodeList.Add(OtherNode);
					}
				}
			}

			BreakPinLinks(*TargetPin, false);
		}
	}
	
	// Send all nodes that lost connections a notification
	for (auto It = NodeList.CreateConstIterator(); It; ++It)
	{
		UEdGraphNode* Node = (*It);
		Node->NodeConnectionListChanged();
	}
#endif	//#if WITH_EDITOR
}

bool UEdGraphSchema::SetNodeMetaData(UEdGraphNode* Node, FName const& KeyValue)
{
#if WITH_METADATA
	if (UPackage* Package = Node->GetOutermost())
	{
		FMetaData& MetaData = Package->GetMetaData();
		MetaData.SetValue(Node, KeyValue, TEXT("true"));
		return true;
	}
#endif
	return false;
}

void UEdGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
#if WITH_EDITOR
	// Copy the old pin links
	TArray<class UEdGraphPin*> OldLinkedTo(TargetPin.LinkedTo);
#endif

	TargetPin.BreakAllPinLinks();

#if WITH_EDITOR
	UEdGraphNode* OwningNode = TargetPin.GetOwningNode();
	TSet<UEdGraphNode*> NodeList;

	// Notify this node
	if (OwningNode != nullptr)
	{
		OwningNode->PinConnectionListChanged(&TargetPin);
		NodeList.Add(OwningNode);
	}

	// As well as all other nodes that were connected
	for (UEdGraphPin* OtherPin : OldLinkedTo)
	{
		if (UEdGraphNode* OtherNode = OtherPin->GetOwningNode())
		{
			OtherNode->PinConnectionListChanged(OtherPin);
			NodeList.Add(OtherNode);
		}
	}

	if (bSendsNodeNotification)
	{
		// Send all nodes that received a new pin connection a notification
		for (UEdGraphNode* Node : NodeList)
		{
			Node->NodeConnectionListChanged();
		}
	}
	
#endif	//#if WITH_EDITOR
}

void UEdGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	SourcePin->BreakLinkTo(TargetPin);

#if WITH_EDITOR
	// get a reference to these now as the following calls can potentially clear the OwningNode (ex: split pins in MakeArray nodes)
	UEdGraphNode* TargetNode = TargetPin->GetOwningNode();
	UEdGraphNode* SourceNode = SourcePin->GetOwningNode();
	
	TargetNode->PinConnectionListChanged(TargetPin);
	SourceNode->PinConnectionListChanged(SourcePin);
	
	TargetNode->NodeConnectionListChanged();
	SourceNode->NodeConnectionListChanged();
#endif	//#if WITH_EDITOR
}

FPinConnectionResponse UEdGraphSchema::MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove, bool bNotifyLinkedNodes) const
{
#if WITH_EDITOR
	ensureMsgf(bIsIntermediateMove || !MoveToPin.GetOwningNode()->GetGraph()->HasAnyFlags(RF_Transient),
		TEXT("When moving to an Intermediate pin, use FKismetCompilerContext::MovePinLinksToIntermediate() instead of UEdGraphSchema::MovePinLinks()"));
#endif // #if WITH_EDITOR

	FPinConnectionResponse FinalResponse = FPinConnectionResponse(CONNECT_RESPONSE_MAKE, FText());
	// First copy the current set of links
	TArray<UEdGraphPin*> CurrentLinks = MoveFromPin.LinkedTo;
	// Then break all links at pin we are moving from
	MoveFromPin.BreakAllPinLinks(false);
	// Try and make each new connection
	for (int32 i=0; i<CurrentLinks.Num(); i++)
	{
		UEdGraphPin* NewLink = CurrentLinks[i];
#if WITH_EDITORONLY_DATA
		// Don't move connections to removed pins
		if (!NewLink->bOrphanedPin)
#endif
		{
			FPinConnectionResponse Response = CanCreateConnection(&MoveToPin, NewLink);
			const bool bCanConnect = Response.CanSafeConnect() || (Response.Response == CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE);

#if WITH_EDITOR
			if (!bCanConnect && bNotifyLinkedNodes)
			{
				// Connection failed, so notify and try again
				if (UEdGraphNode* LinkedToNode = NewLink->GetOwningNodeUnchecked())
				{
					LinkedToNode->PinConnectionListChanged(NewLink);
					Response = CanCreateConnection(&MoveToPin, NewLink);
				}
			}
#endif
			if (bCanConnect)
			{
				if (Response.Response == CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE)
				{
					CreateAutomaticConversionNodeAndConnections(&MoveToPin, NewLink);
				}
				else
				{
					MoveToPin.MakeLinkTo(NewLink);
				}
			}	
			else
			{ 
				FinalResponse = Response;
			}
		}
	}
	// Move over the default values
	MoveToPin.DefaultValue = MoveFromPin.DefaultValue;
	MoveToPin.DefaultObject = MoveFromPin.DefaultObject;
	MoveToPin.DefaultTextValue = MoveFromPin.DefaultTextValue;
	return FinalResponse;
}

FPinConnectionResponse UEdGraphSchema::CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy) const
{
#if WITH_EDITOR
	ensureMsgf(bIsIntermediateCopy || !CopyToPin.GetOwningNode()->GetGraph()->HasAnyFlags(RF_Transient),
		TEXT("When copying to an Intermediate pin, use FKismetCompilerContext::CopyPinLinksToIntermediate() instead of UEdGraphSchema::CopyPinLinks()"));
#endif // #if WITH_EDITOR

	FPinConnectionResponse FinalResponse = FPinConnectionResponse(CONNECT_RESPONSE_MAKE, FText());
	for (int32 i=0; i<CopyFromPin.LinkedTo.Num(); i++)
	{
		UEdGraphPin* NewLink = CopyFromPin.LinkedTo[i];
		FPinConnectionResponse Response = CanCreateConnection(&CopyToPin, NewLink);
#if WITH_EDITORONLY_DATA
		// Don't copy connections to removed pins
		if (!NewLink->bOrphanedPin)
#endif
		{
			if (Response.CanSafeConnect())
			{
				CopyToPin.MakeLinkTo(NewLink);
			}
			else
			{
				FinalResponse = Response;
			}
		}
	}

	CopyToPin.DefaultValue = CopyFromPin.DefaultValue;
	CopyToPin.DefaultObject = CopyFromPin.DefaultObject;
	CopyToPin.DefaultTextValue = CopyFromPin.DefaultTextValue;
	return FinalResponse;
}

#if WITH_EDITORONLY_DATA
FText UEdGraphSchema::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	FText ResultPinName;
	check(Pin != nullptr);
	if (Pin->PinFriendlyName.IsEmpty())
	{
		// We don't want to display "None" for no name
		if (Pin->PinName.IsNone())
		{
			return FText::GetEmpty();
		}
		else
		{
			ResultPinName = FText::FromName(Pin->PinName);
		}
	}
	else
	{
		ResultPinName = Pin->PinFriendlyName;

		bool bShouldUseLocalizedNodeAndPinNames = false;
		GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedNodeAndPinNames"), bShouldUseLocalizedNodeAndPinNames, GEditorSettingsIni);
		if (!bShouldUseLocalizedNodeAndPinNames)
		{
			ResultPinName = FText::FromString(ResultPinName.BuildSourceString());
		}
	}
	return ResultPinName;
}

float UEdGraphSchema::GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const
{
	// The overall 'weight'
	int32 TotalWeight = 0;

	// Combine the actions string, separate with \n so terms don't run into each other, and remove the spaces (incase the user is searching for a variable)
	// In the case of groups containing multiple actions, they will have been created and added at the same place in the code, using the same description
	// and keywords, so we only need to use the first one for filtering.
	const FString& SearchText = InCurrentAction.GetFullSearchText();

	// Setup an array of arrays so we can do a weighted search			
	TArray<FGraphSchemaSearchTextWeightInfo> WeightedArrayList;
	FGraphSchemaSearchWeightModifiers WeightModifiers = GetSearchWeightModifiers();
	FGraphSchemaSearchTextDebugInfo DebugInfo;
	int32 NonLocalizedFirstIndex = CollectSearchTextWeightInfo(InCurrentAction, WeightModifiers, WeightedArrayList, &DebugInfo);

	// Now iterate through all the filter terms and calculate a 'weight' using the values and multipliers
	const FString* EachTerm = nullptr;
	const FString* EachTermSanitized = nullptr;
	for (int32 FilterIndex = 0; FilterIndex < InFilterTerms.Num(); ++FilterIndex)
	{
		EachTerm = &InFilterTerms[FilterIndex];
		EachTermSanitized = &InSanitizedFilterTerms[FilterIndex];
		if (SearchText.Contains(*EachTerm, ESearchCase::CaseSensitive))
		{
			TotalWeight += 2;
		}
		else if (SearchText.Contains(*EachTermSanitized, ESearchCase::CaseSensitive))
		{
			TotalWeight++;
		}
		// Now check the weighted lists	(We could further improve the hit weight by checking consecutive word matches)
		for (int32 iFindCount = 0; iFindCount < WeightedArrayList.Num(); iFindCount++)
		{
			int32 WeightPerList = 0;
			const TArray<FString>& KeywordArray = *WeightedArrayList[iFindCount].Array;
			int32 EachWeight = WeightedArrayList[iFindCount].WeightModifier;
			int32 WholeMatchCount = 0;
			int32 WholeMatchMultiplier = (iFindCount < NonLocalizedFirstIndex) ? WeightModifiers.WholeMatchLocalizedWeightMultiplier : WeightModifiers.WholeMatchWeightMultiplier;

			for (int32 iEachWord = 0; iEachWord < KeywordArray.Num(); iEachWord++)
			{
				// If we get an exact match weight the find count to get exact matches higher priority
				if (KeywordArray[iEachWord].StartsWith(*EachTerm, ESearchCase::CaseSensitive))
				{
					if (iEachWord == 0)
					{
						WeightPerList += EachWeight * WholeMatchMultiplier;
					}
					else
					{
						WeightPerList += EachWeight;
					}
					WholeMatchCount++;
				}
				else if (KeywordArray[iEachWord].Contains(*EachTerm, ESearchCase::CaseSensitive))
				{
					WeightPerList += EachWeight;
				}
				if (KeywordArray[iEachWord].StartsWith(*EachTermSanitized, ESearchCase::CaseSensitive))
				{
					if (iEachWord == 0)
					{
						WeightPerList += EachWeight * WholeMatchMultiplier;
					}
					else
					{
						WeightPerList += EachWeight;
					}
					WholeMatchCount++;
				}
				else if (KeywordArray[iEachWord].Contains(*EachTermSanitized, ESearchCase::CaseSensitive))
				{
					WeightPerList += EachWeight / 2;
				}
			}

			// Increase the weight if theres a larger % of matches in the keyword list
			if (WholeMatchCount != 0)
			{
				int32 PercentAdjust = (100 / KeywordArray.Num()) * WholeMatchCount;
				int32 PercentAdjustedWeight = WeightPerList * PercentAdjust;
				DebugInfo.PercentMatch += (float)KeywordArray.Num() / (float)WholeMatchCount;
				DebugInfo.PercentMatchWeight += (PercentAdjustedWeight - WeightPerList);
				WeightPerList = PercentAdjustedWeight;
			}

			if (WeightedArrayList[iFindCount].DebugWeight)
			{
				*WeightedArrayList[iFindCount].DebugWeight += WeightPerList;
			}
			TotalWeight += WeightPerList;
		}
	}

	PrintSearchTextDebugInfo(InFilterTerms, InCurrentAction, &DebugInfo);
	return TotalWeight;

}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float UEdGraphSchema::GetActionFilteredWeight(const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const
{
	int32 TotalWeight = 0;
	int32 Action = 0;
	if (InCurrentAction.Actions[Action].IsValid() == true)
	{
		return GetActionFilteredWeight(*InCurrentAction.Actions[Action], InFilterTerms, InSanitizedFilterTerms, DraggedFromPins);
	}
	return TotalWeight;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

int32 UEdGraphSchema::CollectSearchTextWeightInfo(const FEdGraphSchemaAction& InCurrentAction, const FGraphSchemaSearchWeightModifiers& InWeightModifiers,
	TArray<FGraphSchemaSearchTextWeightInfo>& OutWeightedArrayList, FGraphSchemaSearchTextDebugInfo* InDebugInfo) const
{
	// First the localized keywords
	OutWeightedArrayList.Add(FGraphSchemaSearchTextWeightInfo(&InCurrentAction.GetLocalizedSearchKeywordsArray(), InWeightModifiers.KeywordWeight, InDebugInfo ? &InDebugInfo->KeywordWeight : nullptr));

	// The localized description
	OutWeightedArrayList.Add(FGraphSchemaSearchTextWeightInfo(&InCurrentAction.GetLocalizedMenuDescriptionArray(), InWeightModifiers.DescriptionWeight, InDebugInfo ? &InDebugInfo->DescriptionWeight : nullptr));

	// The node search localized title weight
	OutWeightedArrayList.Add(FGraphSchemaSearchTextWeightInfo(&InCurrentAction.GetLocalizedSearchTitleArray(), InWeightModifiers.NodeTitleWeight, InDebugInfo ? &InDebugInfo->NodeTitleWeight : nullptr));

	// The localized category
	OutWeightedArrayList.Add(FGraphSchemaSearchTextWeightInfo(&InCurrentAction.GetLocalizedSearchCategoryArray(), InWeightModifiers.CategoryWeight, InDebugInfo ? &InDebugInfo->CategoryWeight : nullptr));

	// First the keywords
	int32 NonLocalizedFirstIndex = OutWeightedArrayList.Add(FGraphSchemaSearchTextWeightInfo(&InCurrentAction.GetSearchKeywordsArray(), InWeightModifiers.KeywordWeight, InDebugInfo ? &InDebugInfo->KeywordWeight : nullptr));

	// The description
	OutWeightedArrayList.Add(FGraphSchemaSearchTextWeightInfo(&InCurrentAction.GetMenuDescriptionArray(), InWeightModifiers.DescriptionWeight, InDebugInfo ? &InDebugInfo->DescriptionWeight : nullptr));

	// The node search title weight
	OutWeightedArrayList.Add(FGraphSchemaSearchTextWeightInfo(&InCurrentAction.GetSearchTitleArray(), InWeightModifiers.NodeTitleWeight, InDebugInfo ? &InDebugInfo->NodeTitleWeight : nullptr));

	// The category
	OutWeightedArrayList.Add(FGraphSchemaSearchTextWeightInfo(&InCurrentAction.GetSearchCategoryArray(), InWeightModifiers.CategoryWeight, InDebugInfo ? &InDebugInfo->CategoryWeight : nullptr));

	return NonLocalizedFirstIndex;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 UEdGraphSchema::CollectSearchTextWeightInfo(const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const FGraphSchemaSearchWeightModifiers& InWeightModifiers, 
													TArray<FGraphSchemaSearchTextWeightInfo>& OutWeightedArrayList, FGraphSchemaSearchTextDebugInfo* InDebugInfo) const
{
	return CollectSearchTextWeightInfo(*InCurrentAction.Actions[0], InWeightModifiers, OutWeightedArrayList, InDebugInfo);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//////////////////////////////////////////////////////////////////////////
/** CVars for tweaking how context menu search picks the best match */
namespace ContextMenuConsoleVariables
{
	/** How much weight the node's title has */
	static float NodeTitleWeight = 1.0f;
	static FAutoConsoleVariableRef CVarNodeTitleWeight(
		TEXT("ContextMenu.NodeTitleWeight"), NodeTitleWeight,
		TEXT("The amount of weight placed on the search items title"),
		ECVF_Default);

	/** Weight used to prefer keywords of actions  */
	static float KeywordWeight = 4.0f;
	static FAutoConsoleVariableRef CVarKeywordWeight(
		TEXT("ContextMenu.KeywordWeight"), KeywordWeight,
		TEXT("The amount of weight placed on search items keyword"),
		ECVF_Default);

	/** Weight used to prefer description of actions  */
	static float DescriptionWeight = 10.0f;
	static FAutoConsoleVariableRef CVarDescriptionWeight(
		TEXT("ContextMenu.DescriptionWeight"), DescriptionWeight,
		TEXT("The amount of weight placed on search items description"),
		ECVF_Default);

	/** Weight that a match to a category search has */
	static float CategoryWeight = 1.0f;
	static FAutoConsoleVariableRef CVarCategoryWeight(
		TEXT("ContextMenu.CategoryWeight"), CategoryWeight,
		TEXT("The amount of weight placed on categories that match what the user has typed in"),
		ECVF_Default);

	/** The multiplier given if there is an exact localized match to the search term */
	static float WholeMatchLocalizedWeightMultiplier = 3.0f;
	static FAutoConsoleVariableRef CVarWholeMatchLocalizedWeightMultiplier(
		TEXT("ContextMenu.WholeMatchLocalizedWeightMultiplier"), WholeMatchLocalizedWeightMultiplier,
		TEXT("The multiplier given if there is an exact localized match to the search term"),
		ECVF_Default);

	/** The multiplier given if there is an exact match to the search term */
	static float WholeMatchWeightMultiplier = 2.0f;
	static FAutoConsoleVariableRef CVarWholeMatchWeightMultiplier(
		TEXT("ContextMenu.WholeMatchWeightMultiplier"), WholeMatchWeightMultiplier,
		TEXT("The multiplier given if there is an exact match to the search term"),
		ECVF_Default);

	/** Increasing this will prefer whole percentage matches when comparing the keyword to what the user has typed in */
	static float PercentageMatchWeightMultiplier = 1.0f;
	static FAutoConsoleVariableRef CVarPercentageMatchWeightMultiplier(
		TEXT("rContextMenu.PercentageMatchWeightMultiplier"), PercentageMatchWeightMultiplier,
		TEXT("A multiplier for how much weight to give something based on the percentage match it is"),
		ECVF_Default);

	/** Enabling the debug printing of context menu selections */
	static bool bPrintDebugInfo = false;
	static FAutoConsoleVariableRef CVarPrintDebugInfo(
		TEXT("ContextMenu.PrintDebugInfo"), bPrintDebugInfo,
		TEXT("Print the debug info about the context menu selection"),
		ECVF_Default);
}

FGraphSchemaSearchWeightModifiers UEdGraphSchema::GetSearchWeightModifiers() const
{
	FGraphSchemaSearchWeightModifiers Modifiers;
	Modifiers.NodeTitleWeight = ContextMenuConsoleVariables::NodeTitleWeight;
	Modifiers.KeywordWeight = ContextMenuConsoleVariables::KeywordWeight;
	Modifiers.DescriptionWeight = ContextMenuConsoleVariables::DescriptionWeight;
	Modifiers.CategoryWeight = ContextMenuConsoleVariables::DescriptionWeight;
	Modifiers.WholeMatchLocalizedWeightMultiplier = ContextMenuConsoleVariables::WholeMatchLocalizedWeightMultiplier;
	Modifiers.WholeMatchWeightMultiplier = ContextMenuConsoleVariables::WholeMatchWeightMultiplier;
	Modifiers.PercentageMatchWeightMultiplier = ContextMenuConsoleVariables::PercentageMatchWeightMultiplier;
	return Modifiers;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UEdGraphSchema::PrintSearchTextDebugInfo(const TArray<FString>& InFilterTerms, const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const FGraphSchemaSearchTextDebugInfo* InDebugInfo) const
{
	if (InCurrentAction.Actions.Num() > 0)
	{
		PrintSearchTextDebugInfo(InFilterTerms, *InCurrentAction.Actions[0], InDebugInfo);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UEdGraphSchema::PrintSearchTextDebugInfo(const TArray<FString>& InFilterTerms, const FEdGraphSchemaAction& InCurrentAction, const FGraphSchemaSearchTextDebugInfo* InDebugInfo) const
{
	if (ContextMenuConsoleVariables::bPrintDebugInfo && InDebugInfo)
	{
		InDebugInfo->Print(InFilterTerms, InCurrentAction);
	}
}

void FGraphSchemaSearchTextDebugInfo::Print(const TArray<FString>& SearchForKeywords, const FEdGraphSchemaAction& Action) const
{
	auto CombineStrings = [](const TArray<FString>& Strings) -> FString
	{
		FString Output;
		for (int32 i = 0; i < Strings.Num(); ++i)
		{
			Output += Strings[i];
			Output += i > 0 ? TEXT(" ") : TEXT("");
		}
		return Output;
	};

	FString SearchText = CombineStrings(SearchForKeywords);
	FString LocalizedTitle = CombineStrings(Action.GetLocalizedSearchTitleArray());
	FString LocalizedKeywords = CombineStrings(Action.GetLocalizedSearchKeywordsArray());
	FString LocalizedDescription = CombineStrings(Action.GetLocalizedMenuDescriptionArray());
	FString LocalizedCategory = CombineStrings(Action.GetLocalizedSearchCategoryArray());
	FString Title = CombineStrings(Action.GetSearchTitleArray());
	FString Keywords = CombineStrings(Action.GetSearchKeywordsArray());
	FString Description = CombineStrings(Action.GetMenuDescriptionArray());
	FString Category = CombineStrings(Action.GetSearchCategoryArray());

	UE_LOG(LogTemp, Log, TEXT("Searching for \"%s\" in [LocTitle:\"%s\" LocKeywords:\"%s\" LocDescription:\"%s\" LocCategory:\"%s\" Title:\"%s\" Keywords:\"%s\" Description:\"%s\" Category:\"%s\"] \
TotalWeight: %.2f | NodeTitleWeight: %.2f | KeywordWeight: %.2f | CategoryWeight: %.2f | PercentMatchWeight: %.2f | ShorterMatchWeight: %.2f"),
		*SearchText, *LocalizedTitle, *LocalizedKeywords, *LocalizedDescription, *LocalizedCategory, *Title, *Keywords, *Description, *Category, 
		TotalWeight, NodeTitleWeight, KeywordWeight, CategoryWeight, PercentMatchWeight, ShorterMatchWeight);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FGraphSchemaSearchTextDebugInfo::Print(const TArray<FString>& SearchForKeywords, const FGraphActionListBuilderBase::ActionGroup& Action) const
{
	return Print(SearchForKeywords, *Action.Actions[0]);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

void UEdGraphSchema::ConstructBasicPinTooltip(UEdGraphPin const& Pin, FText const& PinDescription, FString& TooltipOut) const
{
	TooltipOut = PinDescription.ToString();
}

void UEdGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
#if WITH_EDITOR
	// Run thru all nodes and add any menu items they want to add
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;
		if (Class->IsChildOf(UEdGraphNode::StaticClass()) && !Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
		{
			const UEdGraphNode* ClassCDO = Class->GetDefaultObject<UEdGraphNode>();

			if (ClassCDO->CanCreateUnderSpecifiedSchema(this))
			{
				ClassCDO->GetMenuEntries(ContextMenuBuilder);
			}
		}
	}
#endif
}

void UEdGraphSchema::ReconstructNode(UEdGraphNode& TargetNode, bool bIsBatchRequest/*=false*/) const
{
#if WITH_EDITOR
	TargetNode.ReconstructNode();
#endif	//#if WITH_EDITOR
}

void UEdGraphSchema::SetNodePosition(UEdGraphNode* Node, const FVector2D& Position) const
{
	check(Node);
	Node->Modify();
	Node->NodePosX = Position.X;
	Node->NodePosY = Position.Y;
}

void UEdGraphSchema::SetNodePosition(UEdGraphNode* Node, const FVector2f& Position) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetNodePosition(Node, FVector2D(Position));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FReply UEdGraphSchema::BeginGraphDragAction(TSharedPtr<FEdGraphSchemaAction> InAction) const
{
	return FReply::Unhandled();
}

FReply UEdGraphSchema::BeginGraphDragAction(TSharedPtr<FEdGraphSchemaAction> InAction, const FPointerEvent& MouseEvent) const
{
	return FReply::Unhandled();
}

void UEdGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	DisplayInfo.PlainName = FText::FromString( Graph.GetName() );
	DisplayInfo.DisplayName = DisplayInfo.PlainName;
}

FReply UEdGraphSchema::TrySetGraphCategory(const UEdGraph* InGraph, const FText& InCategory) const
{
	return FReply::Unhandled();
}

#if WITH_EDITOR
TSharedPtr<INameValidatorInterface> UEdGraphSchema::GetNameValidator(const UBlueprint* InBlueprintObj, const FName& InOriginalName, const UStruct* InValidationScope, const FName& InActionTypeId) const
{
	return MakeShareable(new FKismetNameValidator(InBlueprintObj, InOriginalName, InValidationScope));
}
#endif	//#if WITH_EDITOR

void UEdGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	// Instructions: Implement GetParentContextMenuName() with return NAME_None in classes that do not want UEdGraphSchema's menu entries

	// Note: Menu for UEdGraphSchema is registered in an editor module to reduce dependencies in the engine module
}

FName UEdGraphSchema::GetContextMenuName() const
{
	return GetContextMenuName(GetClass());
}

FName UEdGraphSchema::GetParentContextMenuName() const
{
#if WITH_EDITOR
	if (GetClass() != UEdGraphSchema::StaticClass())
	{
		if (UClass* SuperClass = GetClass()->GetSuperClass())
		{
			return GetContextMenuName(SuperClass);
		}
	}
#endif
	return NAME_None;
}

FName UEdGraphSchema::GetContextMenuName(UClass* InClass)
{
#if WITH_EDITOR
	return FName(*(FString("GraphEditor.GraphContextMenu.") + InClass->GetName()));
#else
	return NAME_None;
#endif
}

FString UEdGraphSchema::IsCurrentPinDefaultValid(const UEdGraphPin* Pin) const
{
	return IsPinDefaultValid(Pin, Pin->DefaultValue, Pin->DefaultObject, Pin->DefaultTextValue);
}

#undef LOCTEXT_NAMESPACE
