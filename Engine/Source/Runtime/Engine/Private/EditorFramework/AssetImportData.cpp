// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorFramework/AssetImportData.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/PackageName.h"
#include "Serialization/JsonSerializer.h"
#include "UObject/Package.h"
#include "UObject/AnimPhysObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetImportData)

#if WITH_EDITOR
#include "Editor/EditorPerProjectUserSettings.h"
#endif


// This whole class is compiled out in non-editor
UAssetImportData::UAssetImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
void UAssetImportData::ScriptedAddFilename(const FString& InPath, int32 Index, FString SourceFileLabel)
{
	Modify();
	//Add or update the Source filename
	AddFileName(InPath, Index, SourceFileLabel);

	PostEditChange();
}
#endif //WITH_EDITOR


#if WITH_EDITORONLY_DATA

FOnImportDataChanged UAssetImportData::OnImportDataChanged;

FString FAssetImportInfo::ToJson() const
{
	FString Json;
	Json.Reserve(1024);
	Json += TEXT("[");

	for (int32 Index = 0; Index < SourceFiles.Num(); ++Index)
	{
		Json += FString::Printf(TEXT("{ \"RelativeFilename\" : \"%s\", \"Timestamp\" : \"%" INT64_FMT "\", \"FileMD5\" : \"%s\", \"DisplayLabelName\" : \"%s\" }"),
			*SourceFiles[Index].RelativeFilename,
			SourceFiles[Index].Timestamp.ToUnixTimestamp(),
			*LexToString(SourceFiles[Index].FileHash),
			*SourceFiles[Index].DisplayLabelName
			);

		if (Index != SourceFiles.Num() - 1)
		{
			Json += TEXT(",");
		}
	}

	Json += TEXT("]");
	return Json;
}

TOptional<FAssetImportInfo> FAssetImportInfo::FromJson(FString InJsonString)
{
	// Load json
	TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(MoveTemp(InJsonString));

	TArray<TSharedPtr<FJsonValue>> JSONSourceFiles;
	if (!FJsonSerializer::Deserialize(Reader, JSONSourceFiles))
	{
		return TOptional<FAssetImportInfo>();
	}
	
	FAssetImportInfo Info;

	for (const auto& Value : JSONSourceFiles)
	{
		const TSharedPtr<FJsonObject>& SourceFile = Value->AsObject();
		if (!SourceFile.IsValid())
		{
			continue;
		}

		FString RelativeFilename, TimestampString, MD5String, DisplayLabelName;
		SourceFile->TryGetStringField(TEXT("RelativeFilename"), RelativeFilename);
		SourceFile->TryGetStringField(TEXT("Timestamp"), TimestampString);
		SourceFile->TryGetStringField(TEXT("FileMD5"), MD5String);
		SourceFile->TryGetStringField(TEXT("DisplayLabelName"), DisplayLabelName);

		if (RelativeFilename.IsEmpty())
		{
			continue;
		}

		int64 UnixTimestamp = 0;
		LexFromString(UnixTimestamp, *TimestampString);

		FMD5Hash FileHash;
		LexFromString(FileHash, *MD5String);

		Info.SourceFiles.Emplace(MoveTemp(RelativeFilename), FDateTime::FromUnixTimestamp(UnixTimestamp), FileHash, DisplayLabelName);
	}

	return Info;
}

void UAssetImportData::UpdateFilenameOnly(const FString& InPath)
{
	// Try and retain the MD5 and timestamp if possible
	if (SourceData.SourceFiles.Num() == 1)
	{
		SourceData.SourceFiles[0].RelativeFilename = SanitizeImportFilename(InPath);
	}
	else
	{
		SourceData.SourceFiles.Reset();
		SourceData.SourceFiles.Emplace(SanitizeImportFilename(InPath));
	}
}

void UAssetImportData::UpdateFilenameOnly(const FString& InPath, int32 Index)
{
	if (SourceData.SourceFiles.IsValidIndex(Index))
	{
		SourceData.SourceFiles[Index].RelativeFilename = SanitizeImportFilename(InPath);
	}
	else if(Index == INDEX_NONE)
	{
		UpdateFilenameOnly(InPath);
	}
}

void UAssetImportData::AddFileName(const FString& InPath, int32 Index, FString SourceFileLabel /*= FString()*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetImportData::AddFileName);
	FAssetImportInfo Old = SourceData;

	// Reset our current data
	SourceData.SourceFiles.Reset();

	int32 SourceIndex = 0;
	for (; SourceIndex < Old.SourceFiles.Num(); ++SourceIndex)
	{
		if (SourceIndex == Index)
		{
			SourceData.SourceFiles.Emplace(SanitizeImportFilename(InPath),
				IFileManager::Get().GetTimeStamp(*InPath),
				FMD5Hash::HashFile(*InPath),
				SourceFileLabel);
		}
		else
		{
			SourceData.SourceFiles.Add(Old.SourceFiles[SourceIndex]);
		}
	}
	//If there is now more source file we need to add them
	for (; SourceIndex <= Index; ++SourceIndex)
	{
		if (SourceIndex == Index)
		{
			SourceData.SourceFiles.Emplace(SanitizeImportFilename(InPath),
				IFileManager::Get().GetTimeStamp(*InPath),
				FMD5Hash::HashFile(*InPath),
				SourceFileLabel);
		}
		else
		{
			FString DefaultPath = FString();
			SourceData.SourceFiles.Emplace(SanitizeImportFilename(DefaultPath));
		}
	}

	OnImportDataChanged.Broadcast(Old, this);
}

void UAssetImportData::SetSourceFiles(TArray<FAssetImportInfo::FSourceFile>&& SourceFiles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetImportData::SetSourceFiles);
	FAssetImportInfo Old = SourceData;

	for (FAssetImportInfo::FSourceFile& SourceFile : SourceFiles)
	{
		if (!SourceFile.FileHash.IsValid())
		{
			SourceFile.FileHash = FMD5Hash::HashFile(*SourceFile.RelativeFilename);
		}

		if (SourceFile.Timestamp == FDateTime())
		{
			SourceFile.Timestamp = IFileManager::Get().GetTimeStamp(*SourceFile.RelativeFilename);
		}
	}

	SourceData.SourceFiles = MoveTemp(SourceFiles);

	OnImportDataChanged.Broadcast(Old, this);
}

void UAssetImportData::Update(const FString& InPath, FMD5Hash *Md5Hash/* = nullptr*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetImportData::Update);
	FAssetImportInfo Old = SourceData;
	SourceData.SourceFiles.Reset();
	for (int32 SourceIndex = 0; SourceIndex < Old.SourceFiles.Num(); ++SourceIndex)
	{
		if (SourceIndex == 0)
		{
			SourceData.SourceFiles.Emplace(SanitizeImportFilename(InPath),
				IFileManager::Get().GetTimeStamp(*InPath),
				(Md5Hash != nullptr) ? *Md5Hash : FMD5Hash::HashFile(*InPath));
		}
		else
		{
			SourceData.SourceFiles.Add(Old.SourceFiles[SourceIndex]);
		}
	}
	if (SourceData.SourceFiles.Num() == 0)
	{
		SourceData.SourceFiles.Emplace(SanitizeImportFilename(InPath),
			IFileManager::Get().GetTimeStamp(*InPath),
			(Md5Hash != nullptr) ? *Md5Hash : FMD5Hash::HashFile(*InPath));
	}
	
	OnImportDataChanged.Broadcast(Old, this);
}

//@third party BEGIN SIMPLYGON
void UAssetImportData::Update(const FString& InPath, const FMD5Hash InPreComputedHash)
{
	FAssetImportInfo Old = SourceData;
	SourceData.SourceFiles.Reset();
	for (int32 SourceIndex = 0; SourceIndex < Old.SourceFiles.Num(); ++SourceIndex)
	{
		if (SourceIndex == 0)
		{
			SourceData.SourceFiles.Emplace(SanitizeImportFilename(InPath),
				IFileManager::Get().GetTimeStamp(*InPath),
				InPreComputedHash);
		}
		else
		{
			SourceData.SourceFiles.Add(Old.SourceFiles[SourceIndex]);
		}
	}
	if (SourceData.SourceFiles.Num() == 0)
	{
		SourceData.SourceFiles.Emplace(SanitizeImportFilename(InPath),
			IFileManager::Get().GetTimeStamp(*InPath),
			InPreComputedHash);
	}

	OnImportDataChanged.Broadcast(Old, this);
}
//@third party END SIMPLYGON

#if WITH_EDITOR
FString UAssetImportData::K2_GetFirstFilename() const
{
	return GetFirstFilename();
}
#endif

FString UAssetImportData::GetFirstFilename() const
{
	return SourceData.SourceFiles.Num() > 0 ? ResolveImportFilename(SourceData.SourceFiles[0].RelativeFilename) : FString();
}

#if WITH_EDITOR
TArray<FString> UAssetImportData::K2_ExtractFilenames() const
{
	return ExtractFilenames();
}
#endif

void UAssetImportData::ExtractFilenames(TArray<FString>& AbsoluteFilenames) const
{
	for (const auto& File : SourceData.SourceFiles)
	{
		AbsoluteFilenames.Add(ResolveImportFilename(File.RelativeFilename));
	}
}

TArray<FString> UAssetImportData::ExtractFilenames() const
{
	TArray<FString> Temp;
	ExtractFilenames(Temp);
	return Temp;
}

void UAssetImportData::ExtractDisplayLabels(TArray<FString>& FileDisplayLabels) const
{
	for (const FAssetImportInfo::FSourceFile& SourceFile : SourceData.SourceFiles)
	{
		FileDisplayLabels.Add(SourceFile.DisplayLabelName);
	}
}

FString UAssetImportData::SanitizeImportFilename(const FString& InPath) const
{
	return SanitizeImportFilename(InPath, GetOutermost());
}

FString UAssetImportData::SanitizeImportFilename(const FString& InPath, const UPackage* Outermost)
{
	return SanitizeImportFilename(InPath, Outermost ? Outermost->GetPathName() : FString());
}

FString UAssetImportData::SanitizeImportFilename(const FString& InPath, const FString& PackagePath)
{
	if (!PackagePath.IsEmpty())
	{
		const bool		bIncludeDot = true;
		const FName		MountPoint	= FPackageName::GetPackageMountPoint(PackagePath);
		const FString	PackageFilename = FPackageName::LongPackageNameToFilename(PackagePath, FPaths::GetExtension(InPath, bIncludeDot));
		const FString	AbsolutePath = FPaths::ConvertRelativePathToFull(InPath);

		if ((MountPoint == FName("Engine") && AbsolutePath.StartsWith(FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir()))) ||
			(MountPoint == FName("Game") &&	AbsolutePath.StartsWith(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()))) ||
			(AbsolutePath.StartsWith(FPaths::ConvertRelativePathToFull(FPaths::ProjectPluginsDir()).Append(MountPoint.ToString()))))
		{
			FString RelativePath = InPath;
			FPaths::MakePathRelativeTo(RelativePath, *PackageFilename);
			return RelativePath;
		}
	}

#if WITH_EDITOR
	FString BaseSourceFolder = GetDefault<UEditorPerProjectUserSettings>()->DataSourceFolder.Path;
	if (!BaseSourceFolder.IsEmpty() && FPaths::DirectoryExists(BaseSourceFolder))
	{
		//Make sure the source folder is clean to do relative operation
		if (!BaseSourceFolder.EndsWith(TEXT("/")) && !BaseSourceFolder.EndsWith(TEXT("\\")))
		{
			BaseSourceFolder += TEXT("/");
		}
		//Look if the InPath is relative to the base source path, if yes we will store a relative path to this folder
		FString RelativePath = InPath;
		if (FPaths::MakePathRelativeTo(RelativePath, *BaseSourceFolder))
		{
			//Make sure the path is under the base source folder
			if (!RelativePath.StartsWith(TEXT("..")))
			{
				return RelativePath;
			}
		}
	}
#endif

	return IFileManager::Get().ConvertToRelativePath(*InPath);
}

FString UAssetImportData::ResolveImportFilename(const FString& InRelativePath, const UPackage* Outermost)
{
	if (Outermost)
	{ 
		return ResolveImportFilename(InRelativePath, FStringView(Outermost->GetPathName()));
	}

	return ResolveImportFilename(InRelativePath, FStringView());
}

FString UAssetImportData::ResolveImportFilename(FStringView InRelativePath, FStringView OutermostPath)
{
	if (!OutermostPath.IsEmpty())
	{
		// Relative to the package filename?
		TStringBuilder<256> Path;
		FPathViews::Append(Path, FPathViews::GetPath(FPackageName::LongPackageNameToFilename(FString(OutermostPath))), InRelativePath);

		FPathViews::ToAbsolutePathInline(Path);
		FString ResolvedFile(Path);
		if (FPaths::FileExists(ResolvedFile))
		{
			//FileExist return true when testing Path like c:/../folder1/filename. ConvertRelativePathToFull specify having .. in front of a drive letter is an error.
			//It is relative to package only if the conversion to full path is successful.
			if (ResolvedFile.Find(TEXT("..")) == INDEX_NONE)
			{
				return ResolvedFile;
			}
		}
	}

#if WITH_EDITOR
	FString BaseSourceFolder = GetDefault<UEditorPerProjectUserSettings>()->DataSourceFolder.Path;
	if (!BaseSourceFolder.IsEmpty() && FPaths::DirectoryExists(BaseSourceFolder))
	{
		//Make sure the source folder is clean to do relative operation
		if (!BaseSourceFolder.EndsWith(TEXT("/")) && !BaseSourceFolder.EndsWith(TEXT("\\")))
		{
			BaseSourceFolder += TEXT("/");
		}
		FString FullPath = FPaths::Combine(BaseSourceFolder, InRelativePath);
		if (FPaths::FileExists(FullPath))
		{
			FString FullConvertPath = FPaths::ConvertRelativePathToFull(FullPath);
			if (FullConvertPath.Find(TEXT("..")) == INDEX_NONE)
			{
				return FullConvertPath;
			}
		}
	}
#endif

	// Convert relative paths
	TStringBuilder<256> Path;
	FPathViews::ToAbsolutePath(InRelativePath, Path);
	return FString(Path);
}

FString UAssetImportData::ResolveImportFilename(const FString& InRelativePath) const
{
	return ResolveImportFilename(InRelativePath, GetOutermost());
}

void UAssetImportData::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if (BaseArchive.UEVer() >= VER_UE4_ASSET_IMPORT_DATA_AS_JSON)
	{
		if (!BaseArchive.IsFilterEditorOnly())
		{
			FString Json;
			if (BaseArchive.IsLoading())
			{
				Record << SA_VALUE(TEXT("Json"), Json);
				TOptional<FAssetImportInfo> Copy = FAssetImportInfo::FromJson(MoveTemp(Json));
				if (Copy.IsSet())
				{
					SourceData = MoveTemp(Copy.GetValue());
				}
			}
			else if (BaseArchive.IsSaving())
			{
				Json = SourceData.ToJson();
				Record << SA_VALUE(TEXT("Json"), Json);
			}
		}
	}

	Super::Serialize(Record);

	BaseArchive.UsingCustomVersion(FAnimPhysObjectVersion::GUID);

	if (BaseArchive.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::ThumbnailSceneInfoAndAssetImportDataAreTransactional)
	{
		SetFlags(RF_Transactional);
	}
}

void UAssetImportData::PostLoad()
{
	if (!SourceFilePath_DEPRECATED.IsEmpty() && SourceData.SourceFiles.Num() == 0)
	{
		FDateTime SourceDateTime;
		if (!FDateTime::Parse(SourceFileTimestamp_DEPRECATED,SourceDateTime))
		{
			SourceDateTime = 0;
		}

		SourceData.SourceFiles.Add(FAssetImportInfo::FSourceFile(MoveTemp(SourceFilePath_DEPRECATED),SourceDateTime));

		SourceFilePath_DEPRECATED.Empty();
		SourceFileTimestamp_DEPRECATED.Empty();
	}

	Super::PostLoad();
}

#endif // WITH_EDITORONLY_DATA

