// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementCounterInterface.h"
#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"

#include "Elements/Object/ObjectElementCounterInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementCounterInterface)

void UActorElementCounterInterface::IncrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		UObjectElementCounterInterface::IncrementCounterForObjectClass(Actor, InOutCounter);
	}
}

void UActorElementCounterInterface::DecrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		UObjectElementCounterInterface::DecrementCounterForObjectClass(Actor, InOutCounter);
	}
}

