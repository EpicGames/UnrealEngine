// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "GameFramework/Actor.h"

UE_DEFINE_TYPED_ELEMENT_DATA_RTTI(FActorElementData);

namespace ActorElementDataUtil
{

AActor* GetActorFromHandle(const FTypedElementHandle& InHandle, const bool bSilent)
{
	const FActorElementData* ActorElement = InHandle.GetData<FActorElementData>(bSilent);
	return ActorElement ? ActorElement->ActorWeak.GetEvenIfUnreachable() : nullptr;
}

AActor* GetActorFromHandleChecked(const FTypedElementHandle& InHandle)
{
	const FActorElementData& ActorElement = InHandle.GetDataChecked<FActorElementData>();
	return ActorElement.ActorWeak.GetEvenIfUnreachable();
}

} // namespace ActorElementDataUtil

FActorElementData::FActorElementData()
{
}

FActorElementData::~FActorElementData()
{
}

FActorElementData::FActorElementData(const FActorElementData& Other)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	this->Actor = Other.Actor;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	this->ActorWeak = Other.ActorWeak;
}

FActorElementData::FActorElementData(FActorElementData&& Other)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	this->Actor = MoveTemp(Other.Actor);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	this->ActorWeak = MoveTemp(Other.ActorWeak);
}

FActorElementData& FActorElementData::operator=(const FActorElementData& Other)
{
	if (this != &Other)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		this->Actor = Other.Actor;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		this->ActorWeak = Other.ActorWeak;
	}
	return *this;
}

FActorElementData& FActorElementData::operator=(FActorElementData&& Other)
{
	if (this != &Other)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		this->Actor = Other.Actor;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		this->ActorWeak = Other.ActorWeak;
	}
	return *this;
}

template <>
FString GetTypedElementDebugId<FActorElementData>(const FActorElementData& InElementData)
{
	const AActor* Object = InElementData.ActorWeak.GetEvenIfUnreachable();
	return IsValid(Object)
		? Object->GetFullName()
		: TEXT("null");
}
