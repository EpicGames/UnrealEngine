// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementHierarchyInterface.h"

#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"

#include "Elements/Framework/EngineElementsLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementHierarchyInterface)

void UActorElementHierarchyInterface::GetChildElements(const FTypedElementHandle& InElementHandle, TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
#if WITH_EDITOR
		for (UActorComponent* Component : Actor->GetComponents())
		{
			FTypedElementHandle ComponentHandle = Component ? UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component, bAllowCreate) : FTypedElementHandle();
			if (ComponentHandle)
			{
				OutElementHandles.Add(MoveTemp(ComponentHandle));
			}
		}
#endif	// WITH_EDITOR
	}
}

