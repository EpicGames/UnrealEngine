// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementTedsTypedElementBridgeInterface.h"

#include "DataStorage/Features.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementTedsTypedElementBridgeInterface)

FTedsRowHandle UActorElementTedsTypedElementBridgeInterface::GetRowHandle(const FTypedElementHandle& InElementHandle) const
{
	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);

	using namespace UE::Editor::DataStorage;
	ICompatibilityProvider* CompatibilityProvider = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	RowHandle Handle = CompatibilityProvider->FindRowWithCompatibleObject(Actor);
	
	return FTedsRowHandle(Handle);
}
