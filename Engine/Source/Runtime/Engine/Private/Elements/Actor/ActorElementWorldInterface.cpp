// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementWorldInterface.h"

#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Component/ComponentElementData.h"
#include "Engine/World.h"

#include "ActorEditorUtils.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Volume.h"
#include "ShowFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementWorldInterface)

bool UActorElementWorldInterface::IsTemplateElement(const FTypedElementHandle& InElementHandle)
{
	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	return Actor && Actor->IsTemplate();
}

ULevel* UActorElementWorldInterface::GetOwnerLevel(const FTypedElementHandle& InElementHandle)
{
	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	return Actor ? Actor->GetLevel() : nullptr;
}

UWorld* UActorElementWorldInterface::GetOwnerWorld(const FTypedElementHandle& InElementHandle)
{
	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	return Actor ? Actor->GetWorld() : nullptr;
}

bool UActorElementWorldInterface::GetBounds(const FTypedElementHandle& InElementHandle, FBoxSphereBounds& OutBounds)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		// TODO: This was taken from FActorOrComponent, but AActor has a function to calculate bounds too...
		if (const USceneComponent* RootComponent = Actor->GetRootComponent())
		{
			OutBounds = RootComponent->Bounds;
			return true;
		}
	}

	return false;
}

bool UActorElementWorldInterface::CanMoveElement(const FTypedElementHandle& InElementHandle, const ETypedElementWorldType InWorldType)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
#if WITH_EDITOR
		// The actor cannot be location locked
		if (InWorldType == ETypedElementWorldType::Editor && Actor->IsLockLocation())
		{
			return false;
		}
#endif // WITH_EDITOR

		// If the actor has a root component, but it cannot be moved, then the actor cannot move.
		if (InWorldType == ETypedElementWorldType::Game && Actor->GetRootComponent() && !Actor->IsRootComponentMovable())
		{
			return false;
		}

		return true;
	}

	return false;
}

bool UActorElementWorldInterface::CanScaleElement(const FTypedElementHandle& InElementHandle)
{
#if WITH_EDITOR
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		// The actor must be scalable.
		FProperty* const RelativeScale3DProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeScale3DPropertyName());
		return Actor->CanEditChangeComponent(Actor->GetRootComponent(), RelativeScale3DProperty);
	}
#endif // WITH_EDITOR

	return true;
}

bool UActorElementWorldInterface::GetWorldTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		OutTransform = Actor->GetActorTransform();
		return true;
	}

	return false;
}

bool UActorElementWorldInterface::SetWorldTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		Actor->Modify();
		return Actor->SetActorTransform(InTransform);
	}

	return false;
}

bool UActorElementWorldInterface::GetRelativeTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		if (const USceneComponent* RootComponent = Actor->GetRootComponent())
		{
			OutTransform = RootComponent->GetRelativeTransform();
		}
		else
		{
			OutTransform = FTransform::Identity;
		}
		return true;
	}

	return false;
}

bool UActorElementWorldInterface::SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		if (USceneComponent* RootComponent = Actor->GetRootComponent())
		{
			Actor->Modify();
			RootComponent->SetRelativeTransform(InTransform);
			return true;
		}
	}

	return false;
}

bool UActorElementWorldInterface::FindSuitableTransformAtPoint(const FTypedElementHandle& InElementHandle, const FTransform& InPotentialTransform, FTransform& OutSuitableTransform)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		UWorld* World = Actor->GetWorld();
		const UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Actor->GetRootComponent());

		if (World && PrimComponent && PrimComponent->IsQueryCollisionEnabled())
		{
			const FVector PivotOffset = PrimComponent->Bounds.Origin - Actor->GetActorLocation();

			FVector NewLocation = InPotentialTransform.GetTranslation();
			FRotator NewRotation = InPotentialTransform.Rotator();

			// Apply the pivot offset to the desired location
			NewLocation += PivotOffset;

			// Check if able to find an acceptable destination for this actor that doesn't embed it in world geometry
			if (World->FindTeleportSpot(Actor, NewLocation, NewRotation))
			{
				// Undo the pivot offset
				NewLocation -= PivotOffset;

				OutSuitableTransform = InPotentialTransform;
				OutSuitableTransform.SetTranslation(NewLocation);
				OutSuitableTransform.SetRotation(NewRotation.Quaternion());
				return true;
			}
		}
		else
		{
			OutSuitableTransform = InPotentialTransform;
			return true;
		}
	}

	return false;
}

bool UActorElementWorldInterface::FindSuitableTransformAlongPath(const FTypedElementHandle& InElementHandle, const FVector& InPathStart, const FVector& InPathEnd, const FCollisionShape& InTestShape, TArrayView<const FTypedElementHandle> InElementsToIgnore, FTransform& OutSuitableTransform)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{	
		if (const UWorld* World = Actor->GetWorld())
		{
			FCollisionQueryParams Params(SCENE_QUERY_STAT(FindSuitableTransformAlongPath), false);
			
			// Don't hit ourself
			{
				Params.AddIgnoredActor(Actor);

				TArray<AActor*> ChildActors;
				Actor->GetAllChildActors(ChildActors);
				Params.AddIgnoredActors(ChildActors);
			}

			return UActorElementWorldInterface::FindSuitableTransformAlongPath_WorldSweep(World, InPathStart, InPathEnd, InTestShape, InElementsToIgnore, Params, OutSuitableTransform);
		}
	}

	return false;
}

TArray<FTypedElementHandle> UActorElementWorldInterface::GetSelectionElementsFromSelectionFunction(const FTypedElementHandle& InElementHandle, const FWorldSelectionElementArgs& SelectionArgs, const TFunction<bool(const FTypedElementHandle&, const FWorldSelectionElementArgs&)>& SelectionFunction)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		if ((SelectionArgs.ShowFlags && !SelectionArgs.ShowFlags->Volumes && Actor->IsA(AVolume::StaticClass())) 
			|| FActorEditorUtils::IsABuilderBrush(Actor))
		{
			return {};
		}

		if (!GetOwnerWorld(InElementHandle)->IsEditorWorld())
		{
			if (Actor->IsHidden())
			{
				return {};
			}
		}

		return ITypedElementWorldInterface::GetSelectionElementsFromSelectionFunction(InElementHandle, SelectionArgs, SelectionFunction);
	}

	return {};
}

bool UActorElementWorldInterface::AddIgnoredElementToCollisionQueryParams(const FTypedElementHandle& InElementHandle, FCollisionQueryParams& InOutParams, bool bAlsoIgnoreSubElements)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle, /*bSilent*/true);
	if (!Actor)
	{
		return false;
	}

	TArray<UObject*> IgnoredSourceObjects{ Actor };

	if (bAlsoIgnoreSubElements)
	{
		TArray<AActor*> ChildActors;
		Actor->GetAllChildActors(ChildActors);
		IgnoredSourceObjects.Append(ChildActors);
	}

	InOutParams.AddIgnoredSourceObjects(IgnoredSourceObjects);
	return true;
}

bool UActorElementWorldInterface::FindSuitableTransformAlongPath_WorldSweep(const UWorld* InWorld, 
	const FVector& InPathStart, const FVector& InPathEnd, 
	const FCollisionShape& InTestShape, TArrayView<const FTypedElementHandle> InElementsToIgnore, 
	FCollisionQueryParams& InOutParams, FTransform& OutSuitableTransform)
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	if (ensure(Registry))
	{
		for (const FTypedElementHandle& ElementToIgnore : InElementsToIgnore)
		{
			if (TTypedElement<ITypedElementWorldInterface> WorldInterface = Registry->GetElement<ITypedElementWorldInterface>(ElementToIgnore))
			{
				WorldInterface.AddIgnoredElementToCollisionQueryParams(InOutParams);
			}
		}
	}

	FHitResult Hit(1.0f);
	if (InWorld->SweepSingleByChannel(Hit, InPathStart, InPathEnd, FQuat::Identity, ECC_WorldStatic, InTestShape, InOutParams))
	{
		FVector NewLocation = Hit.Location;
		NewLocation.Z += UE_KINDA_SMALL_NUMBER; // Move the new desired location up by an error tolerance

		//@todo: This doesn't take into account that rotating the actor changes LocationOffset.
		FRotator NewRotation = Hit.Normal.Rotation();
		NewRotation.Pitch -= 90.f;

		OutSuitableTransform.SetTranslation(NewLocation);
		OutSuitableTransform.SetRotation(NewRotation.Quaternion());
		OutSuitableTransform.SetScale3D(FVector::OneVector);
		return true;
	}

	return false;
}

void UActorElementWorldInterface::AddIgnoredCollisionQueryElement(const FTypedElementHandle& InElementHandle, FCollisionQueryParams& InOutParams)
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	if (!ensure(Registry))
	{
		return;
	}

	if (TTypedElement<ITypedElementWorldInterface> WorldInterface = Registry->GetElement<ITypedElementWorldInterface>(InElementHandle))
	{
		// Preserving legacy behavior, where we didn't add child elements:
		const bool bAlsoIgnoreSubElements = false;

		WorldInterface.AddIgnoredElementToCollisionQueryParams(InOutParams, bAlsoIgnoreSubElements);
	}
}

