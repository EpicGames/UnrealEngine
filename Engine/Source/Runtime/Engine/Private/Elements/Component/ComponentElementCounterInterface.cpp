// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementCounterInterface.h"
#include "Elements/Component/ComponentElementData.h"
#include "Components/ActorComponent.h"

#include "Elements/Object/ObjectElementCounterInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentElementCounterInterface)

void UComponentElementCounterInterface::IncrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		UObjectElementCounterInterface::IncrementCounterForObjectClass(Component, InOutCounter);
	}
}

void UComponentElementCounterInterface::DecrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		UObjectElementCounterInterface::DecrementCounterForObjectClass(Component, InOutCounter);
	}
}

