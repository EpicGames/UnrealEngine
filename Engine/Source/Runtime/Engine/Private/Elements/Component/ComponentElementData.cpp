// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementData.h"

#include "Components/ActorComponent.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/Stack.h"

UE_DEFINE_TYPED_ELEMENT_DATA_RTTI(FComponentElementData);

namespace ComponentElementDataUtil
{

UActorComponent* GetComponentFromHandle(const FTypedElementHandle& InHandle, const bool bSilent)
{
	const FComponentElementData* ComponentElement = InHandle.GetData<FComponentElementData>(bSilent);
	return ComponentElement ? ComponentElement->ComponentWeak.GetEvenIfUnreachable() : nullptr;
}

UActorComponent* GetComponentFromHandleChecked(const FTypedElementHandle& InHandle)
{
	const FComponentElementData& ComponentElement = InHandle.GetDataChecked<FComponentElementData>();
	return ComponentElement.ComponentWeak.GetEvenIfUnreachable();
}

} // namespace ComponentElementDataUtil

FComponentElementData::FComponentElementData()
{
}

FComponentElementData::~FComponentElementData()
{
}

FComponentElementData::FComponentElementData(const FComponentElementData& Other)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	this->Component = Other.Component;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	this->ComponentWeak = Other.ComponentWeak;
}

FComponentElementData::FComponentElementData(FComponentElementData&& Other)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	this->Component = MoveTemp(Other.Component);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	this->ComponentWeak = MoveTemp(Other.ComponentWeak);
}

FComponentElementData& FComponentElementData::operator=(const FComponentElementData& Other)
{
	if (this != &Other)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		this->Component = Other.Component;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		this->ComponentWeak = Other.ComponentWeak;
	}
	return *this;
}

FComponentElementData& FComponentElementData::operator=(FComponentElementData&& Other)
{
	if (this != &Other)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		this->Component = Other.Component;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		this->ComponentWeak = Other.ComponentWeak;
	}
	return *this;
}

template <>
FString GetTypedElementDebugId<FComponentElementData>(const FComponentElementData& InElementData)
{
	UActorComponent* Object = InElementData.ComponentWeak.GetEvenIfUnreachable();
	return Object
		? Object->GetFullName()
		: TEXT("null");
}
