// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementHierarchyInterface.h"

#include "Elements/Component/ComponentElementData.h"
#include "Components/ActorComponent.h"

#include "Elements/Framework/EngineElementsLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentElementHierarchyInterface)

FTypedElementHandle UComponentElementHierarchyInterface::GetParentElement(const FTypedElementHandle& InElementHandle, const bool bAllowCreate)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
#if WITH_EDITOR
		if (AActor* OwnerActor = Component->GetOwner())
		{
			return UEngineElementsLibrary::AcquireEditorActorElementHandle(OwnerActor, bAllowCreate);
		}
#endif	// WITH_EDITOR
	}
	return FTypedElementHandle();
}

void UComponentElementHierarchyInterface::GetChildElements(const FTypedElementHandle& InElementHandle, TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		Component->GetComponentChildElements(OutElementHandles, bAllowCreate);
	}
}

