// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementObjectInterface.h"
#include "Elements/Component/ComponentElementData.h"
#include "Components/ActorComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentElementObjectInterface)

UObject* UComponentElementObjectInterface::GetObject(const FTypedElementHandle& InElementHandle)
{
	return ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
}

