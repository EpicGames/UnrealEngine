// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementWorldInterface.h"

#include "Elements/Actor/ActorElementWorldInterface.h"
#include "Elements/Component/ComponentElementData.h"

#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentElementWorldInterface)

bool UComponentElementWorldInterface::CanEditElement(const FTypedElementHandle& InElementHandle)
{
	const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
	return Component && Component->IsEditableWhenInherited();
}

bool UComponentElementWorldInterface::IsTemplateElement(const FTypedElementHandle& InElementHandle)
{
	const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
	return Component && Component->IsTemplate();
}

ULevel* UComponentElementWorldInterface::GetOwnerLevel(const FTypedElementHandle& InElementHandle)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (const AActor* ComponentOwner = Component->GetOwner())
		{
			return ComponentOwner->GetLevel();
		}
	}

	return nullptr;
}

UWorld* UComponentElementWorldInterface::GetOwnerWorld(const FTypedElementHandle& InElementHandle)
{
	const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
	return Component ? Component->GetWorld() : nullptr;
}

bool UComponentElementWorldInterface::GetBounds(const FTypedElementHandle& InElementHandle, FBoxSphereBounds& OutBounds)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (const USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			OutBounds = SceneComponent->Bounds;
			return true;
		}
	}

	return false;
}

bool UComponentElementWorldInterface::CanMoveElement(const FTypedElementHandle& InElementHandle, const ETypedElementWorldType InWorldType)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (const USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			return InWorldType == ETypedElementWorldType::Editor || SceneComponent->Mobility == EComponentMobility::Movable;
		}
	}

	return false;
}

bool UComponentElementWorldInterface::CanScaleElement(const FTypedElementHandle& InElementHandle)
{
#if WITH_EDITOR
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (const USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			FProperty* const RelativeScale3DProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeScale3DPropertyName());
			return SceneComponent->CanEditChange(RelativeScale3DProperty);
		}
	}
#endif // WITH_EDITOR

	return false;
}

bool UComponentElementWorldInterface::GetWorldTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (const USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			OutTransform = SceneComponent->GetComponentTransform();
			return true;
		}
	}

	return false;
}

bool UComponentElementWorldInterface::SetWorldTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			SceneComponent->Modify();
			SceneComponent->SetWorldTransform(InTransform);
			return true;
		}
	}
	
	return false;
}

bool UComponentElementWorldInterface::GetRelativeTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (const USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			OutTransform = SceneComponent->GetRelativeTransform();
			return true;
		}
	}

	return false;
}

bool UComponentElementWorldInterface::SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			SceneComponent->Modify();
			SceneComponent->SetRelativeTransform(InTransform);
			return true;
		}
	}

	return false;
}

bool UComponentElementWorldInterface::FindSuitableTransformAlongPath(const FTypedElementHandle& InElementHandle, const FVector& InPathStart, const FVector& InPathEnd, const FCollisionShape& InTestShape, TArrayView<const FTypedElementHandle> InElementsToIgnore, FTransform& OutSuitableTransform)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (const UWorld* World = Component->GetWorld())
		{
			FCollisionQueryParams Params(SCENE_QUERY_STAT(FindSuitableTransformAlongPath), false);

			// Don't hit ourself
			if (const UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component))
			{
				Params.AddIgnoredComponent(PrimComponent);
			}

			return UActorElementWorldInterface::FindSuitableTransformAlongPath_WorldSweep(World, InPathStart, InPathEnd, InTestShape, InElementsToIgnore, Params, OutSuitableTransform);
		}
	}

	return false;
}

bool UComponentElementWorldInterface::AddIgnoredElementToCollisionQueryParams(const FTypedElementHandle& InElementHandle, FCollisionQueryParams& InOutParams, bool bAlsoIgnoreSubElements)
{
	UActorComponent* Component = Cast<USceneComponent>(ComponentElementDataUtil::GetComponentFromHandle(InElementHandle));

	// Only primitive components are able to be ignored themselves, but scene components
	// could have children that might be ignorable. 

	TArray<UPrimitiveComponent*> ComponentsToIgnore;
	if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
	{
		ComponentsToIgnore.Add(PrimitiveComponent);
	}

	if (bAlsoIgnoreSubElements)
	{
		TArray<USceneComponent*> ChildComponents;
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			SceneComponent->GetChildrenComponents(true, ChildComponents);
		}

		for (USceneComponent* Child : ChildComponents)
		{
			if (UPrimitiveComponent* CastChild = Cast<UPrimitiveComponent>(Child))
			{
				ComponentsToIgnore.Add(CastChild);
			}
		}
	}

	if (!ComponentsToIgnore.IsEmpty())
	{
		InOutParams.AddIgnoredComponents(ComponentsToIgnore);
		return true;
	}
	return false;
}

