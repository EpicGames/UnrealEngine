// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/EngineElements.h"
#include "Elements/Framework/TypedElementRegistry.h"

#include "Elements/Object/ObjectElementData.h"
#include "Elements/Object/ObjectElementAssetDataInterface.h"
#include "Elements/Object/ObjectElementObjectInterface.h"
#include "Elements/Object/ObjectElementCounterInterface.h"
#include "Elements/Object/ObjectElementSelectionInterface.h"

#include "Elements/Actor/ActorElementData.h"
#include "Elements/Actor/ActorElementAssetDataInterface.h"
#include "Elements/Actor/ActorElementHierarchyInterface.h"
#include "Elements/Actor/ActorElementObjectInterface.h"
#include "Elements/Actor/ActorElementCounterInterface.h"
#include "Elements/Actor/ActorElementWorldInterface.h"
#include "Elements/Actor/ActorElementSelectionInterface.h"

#include "Elements/Component/ComponentElementData.h"
#include "Elements/Component/ComponentElementHierarchyInterface.h"
#include "Elements/Component/ComponentElementObjectInterface.h"
#include "Elements/Component/ComponentElementCounterInterface.h"
#include "Elements/Component/ComponentElementWorldInterface.h"
#include "Elements/Component/ComponentElementSelectionInterface.h"

#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Elements/SMInstance/SMInstanceElementHierarchyInterface.h"
#include "Elements/SMInstance/SMInstanceElementWorldInterface.h"
#include "Elements/SMInstance/SMInstanceElementSelectionInterface.h"
#include "Elements/SMInstance/SMInstanceElementAssetDataInterface.h"
#include "Elements/SMInstance/SMInstanceElementPrimitiveCustomDataInterface.h"

#include "Modules/ModuleManager.h"

FSimpleMulticastDelegate OnRegisterEngineElementsDelegate;

void RegisterEngineObjectElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	Registry->RegisterElementType<FObjectElementData, true>(NAME_Object);
	Registry->RegisterElementInterface<ITypedElementAssetDataInterface>(NAME_Object, NewObject<UObjectElementAssetDataInterface>());
	Registry->RegisterElementInterface<ITypedElementObjectInterface>(NAME_Object, NewObject<UObjectElementObjectInterface>());
	Registry->RegisterElementInterface<ITypedElementCounterInterface>(NAME_Object, NewObject<UObjectElementCounterInterface>());
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_Object, NewObject<UObjectElementSelectionInterface>());
}

void RegisterEngineActorElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	Registry->RegisterElementType<FActorElementData, true>(NAME_Actor);
	Registry->RegisterElementInterface<ITypedElementAssetDataInterface>(NAME_Actor, NewObject<UActorElementAssetDataInterface>());
	Registry->RegisterElementInterface<ITypedElementHierarchyInterface>(NAME_Actor, NewObject<UActorElementHierarchyInterface>());
	Registry->RegisterElementInterface<ITypedElementObjectInterface>(NAME_Actor, NewObject<UActorElementObjectInterface>());
	Registry->RegisterElementInterface<ITypedElementCounterInterface>(NAME_Actor, NewObject<UActorElementCounterInterface>());
	Registry->RegisterElementInterface<ITypedElementWorldInterface>(NAME_Actor, NewObject<UActorElementWorldInterface>());
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_Actor, NewObject<UActorElementSelectionInterface>());
}

void RegisterEngineComponentElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	Registry->RegisterElementType<FComponentElementData, true>(NAME_Components);
	Registry->RegisterElementInterface<ITypedElementHierarchyInterface>(NAME_Components, NewObject<UComponentElementHierarchyInterface>());
	Registry->RegisterElementInterface<ITypedElementObjectInterface>(NAME_Components, NewObject<UComponentElementObjectInterface>());
	Registry->RegisterElementInterface<ITypedElementCounterInterface>(NAME_Components, NewObject<UComponentElementCounterInterface>());
	Registry->RegisterElementInterface<ITypedElementWorldInterface>(NAME_Components, NewObject<UComponentElementWorldInterface>());
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_Components, NewObject<UComponentElementSelectionInterface>());
}

void RegisterEngineSMInstanceElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	Registry->RegisterElementType<FSMInstanceElementData, true>(NAME_SMInstance);
	Registry->RegisterElementInterface<ITypedElementHierarchyInterface>(NAME_SMInstance, NewObject<USMInstanceElementHierarchyInterface>());
	Registry->RegisterElementInterface<ITypedElementWorldInterface>(NAME_SMInstance, NewObject<USMInstanceElementWorldInterface>());
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_SMInstance, NewObject<USMInstanceElementSelectionInterface>());
	Registry->RegisterElementInterface<ITypedElementAssetDataInterface>(NAME_SMInstance, NewObject<USMInstanceElementAssetDataInterface>());
	Registry->RegisterElementInterface<ITypedElementPrimitiveCustomDataInterface>(NAME_SMInstance, NewObject<USMInstanceElementPrimitiveCustomDataInterface>());
}

void RegisterEngineElements()
{
	// Ensure the framework and base interfaces are also loaded
	FModuleManager::Get().LoadModuleChecked("TypedElementFramework");
	FModuleManager::Get().LoadModuleChecked("TypedElementRuntime");

	RegisterEngineObjectElements();
	RegisterEngineActorElements();
	RegisterEngineComponentElements();
	RegisterEngineSMInstanceElements();

	OnRegisterEngineElementsDelegate.Broadcast();
}
