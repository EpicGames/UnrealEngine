// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/EngineElementsLibrary.h"

#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementOwnerStore.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"

#include "Elements/Object/ObjectElementData.h"

#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"

#include "Elements/Component/ComponentElementData.h"

#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Components/InstancedStaticMeshComponent.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(EngineElementsLibrary)


// The editor requires ref-counting for object replacement to function correctly
static_assert(!WITH_EDITOR || UE_TYPED_ELEMENT_HAS_REFCOUNTING, "The editor requires that ref-counting is enabled for typed elements!");

namespace EngineElementsLibraryUtil
{
template <typename ObjectClass, typename ElementDataType>
TTypedElementOwner<ElementDataType> CreateTypedElement(const ObjectClass* Object, const FName ElementTypeName)
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	TTypedElementOwner<ElementDataType> TypedElement;
	if (ensureMsgf(Registry, TEXT("Typed element was requested for '%s' before the registry was available! This usually means that NewObject was used instead of CreateDefaultSubobject during CDO construction."), *Object->GetPathName()))
	{
		checkf(!Object->HasAnyFlags(RF_BeginDestroyed), TEXT("Typed element was requested for an object that is being destroyed!"));
		TypedElement = Registry->CreateElement<ElementDataType>(ElementTypeName);
	}

	return TypedElement;
}

#if WITH_EDITOR
template <typename ObjectClass, typename ElementDataType>
void CreateEditorTypedElement(const ObjectClass* Object, TTypedElementOwnerStore<ElementDataType, const ObjectClass*>& ElementOwnerStore, TFunctionRef<TTypedElementOwner<ElementDataType>(const ObjectClass*)> CreateElement)
{
	if (GIsEditor && !Object->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		ElementOwnerStore.FindOrRegisterElementOwner(Object, [Object, &CreateElement]() { return CreateElement(Object); });
	}
}

template <typename ObjectClass, typename ElementDataType>
void DestroyEditorTypedElement(const ObjectClass* Object, TTypedElementOwnerStore<ElementDataType, const ObjectClass*>& ElementOwnerStore, TFunctionRef<void(const ObjectClass*, TTypedElementOwner<ElementDataType>&)> DestroyElement)
{
	if (TTypedElementOwner<ElementDataType> EditorElement = ElementOwnerStore.UnregisterElementOwner(Object))
	{
		DestroyElement(Object, EditorElement);
	}
}

template <typename ObjectClass, typename ElementDataType>
FTypedElementHandle AcquireEditorTypedElementHandle(const ObjectClass* Object, TTypedElementOwnerStore<ElementDataType, const ObjectClass*>& ElementOwnerStore, TFunctionRef<TTypedElementOwner<ElementDataType>(const ObjectClass*)> CreateElement, const bool bAllowCreate)
{
	if (GIsEditor)
	{
		TTypedElementOwnerScopedAccess<ElementDataType> EditorElement = bAllowCreate
			? ElementOwnerStore.FindOrRegisterElementOwner(Object, [Object, &CreateElement](){ return CreateElement(Object); })
			: ElementOwnerStore.FindElementOwner(Object);
		if (EditorElement)
		{
			return EditorElement->AcquireHandle();
		}
	}

	return FTypedElementHandle();
}

template <typename ObjectClass>
TMap<const ObjectClass*, const ObjectClass*> CalculatePotentialObjectReplacements(const TMap<UObject*, UObject*>& ReplacementObjects)
{
	TMap<const ObjectClass*, const ObjectClass*> PotentialObjectReplacements;

	for (const TTuple<UObject*, UObject*>& ReplacementObjectPair : ReplacementObjects)
	{
		if (const ObjectClass* OldObject = Cast<ObjectClass>(ReplacementObjectPair.Key))
		{
			const ObjectClass* NewObject = Cast<ObjectClass>(ReplacementObjectPair.Value);
			PotentialObjectReplacements.Add(OldObject, NewObject);
		}
	}

	return PotentialObjectReplacements;
}

template <typename ElementDataType, typename KeyDataType>
void ReplaceEditorTypedElementHandles(TArray<FTypedElementHandle>& OutUpdatedElements, const TMap<KeyDataType, KeyDataType>& ReplacementKeys, TTypedElementOwnerStore<ElementDataType, KeyDataType>& ElementOwnerStore, TFunctionRef<void(KeyDataType, TTypedElementOwner<ElementDataType>&)> UpdateElement, TFunctionRef<void(KeyDataType, TTypedElementOwner<ElementDataType>&)> DestroyElement)
{
	for (const TTuple<KeyDataType, KeyDataType>& ReplacementKeyPair : ReplacementKeys)
	{
		const KeyDataType& OldKey = ReplacementKeyPair.Key;
		const KeyDataType& NewKey = ReplacementKeyPair.Value;

		// We only need to attempt replacement if we actually have an element for the old object...
		if (TTypedElementOwner<ElementDataType> OldEditorElement = ElementOwnerStore.UnregisterElementOwner(OldKey))
		{
			if (OldEditorElement.Private_GetInternalData()->GetRefCount() > 1)
			{
				if (NewKey)
				{
					// The old element has external references, so we're going to destroy the new element (if any) and re-point the old element to the new object...
					// Note: This requires that the new element has no external references - if both old and new elements have external references then we'll need to support redirection at the element level!
					if (TTypedElementOwner<ElementDataType> NewEditorElement = ElementOwnerStore.UnregisterElementOwner(NewKey))
					{
						if (NewEditorElement.Private_GetInternalData()->GetRefCount() > 1)
						{
							// Both elements currently have external references, so try and update anything referencing the new element so that it references the old one instead...
							// Note: If we had redirection at the element level, then we'd try and do this redirection from old->new instead
							TTuple<FTypedElementHandle, FTypedElementHandle> ElementRedirect = MakeTuple(NewEditorElement.AcquireHandle(), OldEditorElement.AcquireHandle());
							UTypedElementRegistry::GetInstance()->OnElementReplaced().Broadcast(MakeArrayView(&ElementRedirect, 1));
						}

						checkf(NewEditorElement.Private_GetInternalData()->GetRefCount() <= 1, TEXT("The old and new element both have external references! Replacing these will require support for redirection at the element level!"));
						DestroyElement(NewKey, NewEditorElement);
					}

					UpdateElement(NewKey, OldEditorElement);
					OutUpdatedElements.Emplace(OldEditorElement.AcquireHandle());
					ElementOwnerStore.RegisterElementOwner(NewKey, MoveTemp(OldEditorElement));
				}
				else
				{
					// The object has been redirected to null, so try and clear any external references to the old element...
					TTuple<FTypedElementHandle, FTypedElementHandle> ElementRedirect = MakeTuple(OldEditorElement.AcquireHandle(), FTypedElementHandle());
					UTypedElementRegistry::GetInstance()->OnElementReplaced().Broadcast(MakeArrayView(&ElementRedirect, 1));
					DestroyElement(OldKey, OldEditorElement);
				}
			}
			else
			{
				// The old element has no external references, so we can just destroy it...
				DestroyElement(OldKey, OldEditorElement);
			}
		}
	}
}
#endif
} // namespace EngineElementsLibraryUtil

#if WITH_EDITOR
TTypedElementOwnerStore<FObjectElementData, const UObject*> GObjectElementOwnerStore;
TTypedElementOwnerStore<FActorElementData, const AActor*> GActorElementOwnerStore;
TTypedElementOwnerStore<FComponentElementData, const UActorComponent*> GComponentElementOwnerStore;
TTypedElementOwnerStore<FSMInstanceElementData, FSMInstanceElementId> GSMInstanceElementOwnerStore;
#endif

UEngineElementsLibrary::UEngineElementsLibrary()
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// The editor may replace objects and perform fix-up from old->new, so we need to keep any object-based elements in-sync too
		FCoreUObjectDelegates::OnObjectsReplaced.AddStatic(&UEngineElementsLibrary::OnObjectsReplaced);

		// UObject exists inside CoreUObject, so it cannot call through directly to clean-up any element handles that have been created
		// Instead we rely on this GC hook to clean-up any element handles for unreachable objects prior to them being destroyed
		FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy.AddStatic(&UEngineElementsLibrary::DestroyUnreachableEditorObjectElements);

		// Static Mesh Instances are unmapped when removed from their owner component, so we must also destroy any corresponding element handle when that happens
		FSMInstanceElementIdMap::Get().OnInstanceRemoved().AddStatic([](const FSMInstanceElementId& SMInstanceElementId, int32 InstanceIndex) { UEngineElementsLibrary::DestroyEditorSMInstanceElement(SMInstanceElementId); });
	}
#endif
}

#if WITH_EDITOR
void UEngineElementsLibrary::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementObjects)
{
	TArray<FTypedElementHandle> UpdatedElements;

	{
		const TMap<const UObject*, const UObject*> PotentialObjectReplacements = EngineElementsLibraryUtil::CalculatePotentialObjectReplacements<UObject>(InReplacementObjects);
		ReplaceEditorObjectElementHandlesImpl(PotentialObjectReplacements, UpdatedElements);
	}

	{
		const TMap<const AActor*, const AActor*> PotentialActorReplacements = EngineElementsLibraryUtil::CalculatePotentialObjectReplacements<AActor>(InReplacementObjects);
		ReplaceEditorActorElementHandlesImpl(PotentialActorReplacements, UpdatedElements);
	}

	{
		const TMap<const UActorComponent*, const UActorComponent*> PotentialComponentReplacements = EngineElementsLibraryUtil::CalculatePotentialObjectReplacements<UActorComponent>(InReplacementObjects);
		ReplaceEditorComponentElementHandlesImpl(PotentialComponentReplacements, UpdatedElements);
	}

	{
		TMap<FSMInstanceElementId, FSMInstanceElementId> PotentialSMInstanceReplacements;

		FSMInstanceElementIdMap& SMInstanceElementIdMap = FSMInstanceElementIdMap::Get();
		for (const TTuple<UObject*, UObject*>& ReplacementObjectPair : InReplacementObjects)
		{
			if (UInstancedStaticMeshComponent* OldISMComponent = Cast<UInstancedStaticMeshComponent>(ReplacementObjectPair.Key))
			{
				const TArray<FSMInstanceElementId> OldSMInstanceElementIds = SMInstanceElementIdMap.GetSMInstanceElementIdsForComponent(OldISMComponent);
				PotentialSMInstanceReplacements.Reserve(PotentialSMInstanceReplacements.Num() + OldSMInstanceElementIds.Num());

				if (UInstancedStaticMeshComponent* NewISMComponent = Cast<UInstancedStaticMeshComponent>(ReplacementObjectPair.Value))
				{
					// Attempt to ensure that the old IDs are re-used on the new component
					// This is required so that in-memory stored references (eg, undo/redo) map correctly when using the new component instance
					SMInstanceElementIdMap.OnComponentReplaced(OldISMComponent, NewISMComponent);

					for (const FSMInstanceElementId& OldSMInstanceElementId : OldSMInstanceElementIds)
					{
						PotentialSMInstanceReplacements.Add(MakeTuple(OldSMInstanceElementId, FSMInstanceElementId{ NewISMComponent, OldSMInstanceElementId.InstanceId }));
					}
				}
				else
				{
					for (const FSMInstanceElementId& OldSMInstanceElementId : OldSMInstanceElementIds)
					{
						PotentialSMInstanceReplacements.Add(MakeTuple(OldSMInstanceElementId, FSMInstanceElementId()));
					}
				}
			}
		}

		ReplaceEditorSMInstanceElementHandlesImpl(PotentialSMInstanceReplacements, UpdatedElements);
	}

	if (UpdatedElements.Num() > 0)
	{
		UTypedElementRegistry::GetInstance()->OnElementUpdated().Broadcast(UpdatedElements);
	}
}

void UEngineElementsLibrary::ReplaceEditorObjectElementHandlesImpl(const TMap<const UObject*, const UObject*>& ReplacementObjects, TArray<FTypedElementHandle>& OutUpdatedElements)
{
	EngineElementsLibraryUtil::ReplaceEditorTypedElementHandles<FObjectElementData, const UObject*>(OutUpdatedElements, ReplacementObjects, GObjectElementOwnerStore, [](const UObject* InObject, TTypedElementOwner<FObjectElementData>& InOutObjectElement)
	{
		InOutObjectElement.GetDataChecked().Object = const_cast<UObject*>(InObject);
	}, &UEngineElementsLibrary::DestroyObjectElement);
}

void UEngineElementsLibrary::ReplaceEditorActorElementHandlesImpl(const TMap<const AActor*, const AActor*>& ReplacementActors, TArray<FTypedElementHandle>& OutUpdatedElements)
{
	EngineElementsLibraryUtil::ReplaceEditorTypedElementHandles<FActorElementData, const AActor*>(OutUpdatedElements, ReplacementActors, GActorElementOwnerStore, [](const AActor* InActor, TTypedElementOwner<FActorElementData>& InOutActorElement)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InOutActorElement.GetDataChecked().Actor = const_cast<AActor*>(InActor);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		InOutActorElement.GetDataChecked().ActorWeak = const_cast<AActor*>(InActor);
	}, &UEngineElementsLibrary::DestroyActorElement);
}

void UEngineElementsLibrary::ReplaceEditorComponentElementHandlesImpl(const TMap<const UActorComponent*, const UActorComponent*>& ReplacementComponents, TArray<FTypedElementHandle>& OutUpdatedElements)
{
	EngineElementsLibraryUtil::ReplaceEditorTypedElementHandles<FComponentElementData, const UActorComponent*>(OutUpdatedElements, ReplacementComponents, GComponentElementOwnerStore, [](const UActorComponent* InComponent, TTypedElementOwner<FComponentElementData>& InOutComponentElement)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InOutComponentElement.GetDataChecked().Component = const_cast<UActorComponent*>(InComponent);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		InOutComponentElement.GetDataChecked().ComponentWeak = const_cast<UActorComponent*>(InComponent);
	}, &UEngineElementsLibrary::DestroyComponentElement);
}

void UEngineElementsLibrary::ReplaceEditorSMInstanceElementHandlesImpl(const TMap<FSMInstanceElementId, FSMInstanceElementId>& ReplacementSMInstanceIds, TArray<FTypedElementHandle>& OutUpdatedElements)
{
	EngineElementsLibraryUtil::ReplaceEditorTypedElementHandles<FSMInstanceElementData, FSMInstanceElementId>(OutUpdatedElements, ReplacementSMInstanceIds, GSMInstanceElementOwnerStore, [](FSMInstanceElementId InSMInstanceElementId, TTypedElementOwner<FSMInstanceElementData>& InOutSMInstanceElement)
	{
		InOutSMInstanceElement.GetDataChecked().InstanceElementId = InSMInstanceElementId;
	}, &UEngineElementsLibrary::DestroySMInstanceElement);
}
#endif

TTypedElementOwner<FObjectElementData> UEngineElementsLibrary::CreateObjectElement(const UObject* InObject)
{
	TTypedElementOwner<FObjectElementData> ObjectElement = EngineElementsLibraryUtil::CreateTypedElement<UObject, FObjectElementData>(InObject, NAME_Object);
	if (ObjectElement)
	{
		ObjectElement.GetDataChecked().Object = const_cast<UObject*>(InObject);
	}
	return ObjectElement;
}

void UEngineElementsLibrary::DestroyObjectElement(const UObject* InObject, TTypedElementOwner<FObjectElementData>& InOutObjectElement)
{
	if (InOutObjectElement)
	{
		checkf(InOutObjectElement.GetDataChecked().Object == InObject, TEXT("Object element was not for this object instance! %s"), *InObject->GetPathName());
		UTypedElementRegistry::GetInstance()->DestroyElement(InOutObjectElement);
	}
}

#if WITH_EDITOR
void UEngineElementsLibrary::CreateEditorObjectElement(const UObject* Object)
{
	EngineElementsLibraryUtil::CreateEditorTypedElement<UObject, FObjectElementData>(Object, GObjectElementOwnerStore, &UEngineElementsLibrary::CreateObjectElement);
}

void UEngineElementsLibrary::DestroyEditorObjectElement(const UObject* Object)
{
	EngineElementsLibraryUtil::DestroyEditorTypedElement<UObject, FObjectElementData>(Object, GObjectElementOwnerStore, &UEngineElementsLibrary::DestroyObjectElement);
}

void UEngineElementsLibrary::DestroyUnreachableEditorObjectElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	auto IsObjectElementUnreachable = [Registry](const TTypedElementOwner<FObjectElementData>& EditorElement)
	{
		// In the case we are shutting down, the registry may have already been cleaned up when we get here via UTypedElementRegistry::FinishDestroy,
		// and our get checked element access will fail. We always return true if the registry is null, so that the elements can be destroyed during shutdown GC.
		if (!Registry)
		{
			return true;
		}

		return EditorElement.GetDataChecked().Object->IsUnreachable();
	};

	auto DestroyUnreachableObjectElement = [Registry](TTypedElementOwner<FObjectElementData>&& EditorElement)
	{
		if (Registry)
		{
			Registry->DestroyElement(EditorElement);
		}
		else
		{
			EditorElement.Private_DestroyNoRef();
		}
	};

	GObjectElementOwnerStore.UnregisterElementOwners(IsObjectElementUnreachable, DestroyUnreachableObjectElement);
}

FTypedElementHandle UEngineElementsLibrary::AcquireEditorObjectElementHandle(const UObject* Object, const bool bAllowCreate)
{
	return EngineElementsLibraryUtil::AcquireEditorTypedElementHandle<UObject, FObjectElementData>(Object, GObjectElementOwnerStore, &UEngineElementsLibrary::CreateObjectElement, bAllowCreate);
}

FScriptTypedElementHandle UEngineElementsLibrary::K2_AcquireEditorObjectElementHandle(const UObject* Object, const bool bAllowCreate)
{
	if (!Object)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot acquire an editor object element handle from a nullptr object."), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	return UTypedElementRegistry::GetInstance()->CreateScriptHandle(AcquireEditorObjectElementHandle(Object , bAllowCreate).GetId());
}

void UEngineElementsLibrary::ReplaceEditorObjectElementHandles(const TMap<const UObject*, const UObject*>& ReplacementObjects)
{
	TArray<FTypedElementHandle> UpdatedElements;
	ReplaceEditorObjectElementHandlesImpl(ReplacementObjects, UpdatedElements);
	if (UpdatedElements.Num() > 0)
	{
		UTypedElementRegistry::GetInstance()->OnElementUpdated().Broadcast(UpdatedElements);
	}
}
#endif

TTypedElementOwner<FActorElementData> UEngineElementsLibrary::CreateActorElement(const AActor* InActor)
{
	TTypedElementOwner<FActorElementData> ActorElement = EngineElementsLibraryUtil::CreateTypedElement<AActor, FActorElementData>(InActor, NAME_Actor);
	if (ActorElement)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ActorElement.GetDataChecked().Actor = const_cast<AActor*>(InActor);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		ActorElement.GetDataChecked().ActorWeak = const_cast<AActor*>(InActor);
	}
	
	RegisterActorElement(InActor);

	return ActorElement;
}

void UEngineElementsLibrary::DestroyActorElement(const AActor* InActor, TTypedElementOwner<FActorElementData>& InOutActorElement)
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	UnregisterActorElement(InActor);

	if (InOutActorElement)
	{
		checkf(InOutActorElement.GetDataChecked().ActorWeak.GetEvenIfUnreachable() == InActor, TEXT("Actor element was not for this actor instance! %s"), *InActor->GetPathName());
		UTypedElementRegistry::GetInstance()->DestroyElement(InOutActorElement);
	}
}

void UEngineElementsLibrary::RegisterActorElement(const AActor* InActor)
{
	using namespace UE::Editor::DataStorage;
	ICompatibilityProvider* Storage = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	RowHandle AddedObjectRow = InvalidRowHandle;
	if (Storage)
	{
		AddedObjectRow = Storage->AddCompatibleObject(const_cast<AActor*>(InActor));
		constexpr bool bIncludeChildActors = false;
		InActor->ForEachComponent(bIncludeChildActors, [&Storage](UActorComponent* Component)
		{
			Storage->AddCompatibleObject(const_cast<UActorComponent*>(Component));
		});
	}
}

void UEngineElementsLibrary::UnregisterActorElement(const AActor* InActor)
{
	using namespace UE::Editor::DataStorage;
	if (ICompatibilityProvider* Storage = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		Storage->RemoveCompatibleObject(const_cast<AActor*>(InActor));
	}
}

#if WITH_EDITOR
void UEngineElementsLibrary::CreateEditorActorElement(const AActor* Actor)
{
	if (Actor->GetLevel())
	{
		EngineElementsLibraryUtil::CreateEditorTypedElement<AActor, FActorElementData>(Actor, GActorElementOwnerStore, &UEngineElementsLibrary::CreateActorElement);
	}
}

void UEngineElementsLibrary::DestroyEditorActorElement(const AActor* Actor)
{
	EngineElementsLibraryUtil::DestroyEditorTypedElement<AActor, FActorElementData>(Actor, GActorElementOwnerStore, &UEngineElementsLibrary::DestroyActorElement);
}

FTypedElementHandle UEngineElementsLibrary::AcquireEditorActorElementHandle(const AActor* Actor, const bool bAllowCreate)
{
	return EngineElementsLibraryUtil::AcquireEditorTypedElementHandle<AActor, FActorElementData>(Actor, GActorElementOwnerStore, &UEngineElementsLibrary::CreateActorElement, bAllowCreate);
}

FScriptTypedElementHandle UEngineElementsLibrary::K2_AcquireEditorActorElementHandle(const AActor* Actor, const bool bAllowCreate)
{
	if (!Actor)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot acquire an editor actor element handle from a nullptr actor."), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	return UTypedElementRegistry::GetInstance()->CreateScriptHandle(AcquireEditorActorElementHandle(Actor, bAllowCreate).GetId());
}

void UEngineElementsLibrary::ReplaceEditorActorElementHandles(const TMap<const AActor*, const AActor*>& ReplacementActors)
{
	TArray<FTypedElementHandle> UpdatedElements;
	ReplaceEditorActorElementHandlesImpl(ReplacementActors, UpdatedElements);
	if (UpdatedElements.Num() > 0)
	{
		UTypedElementRegistry::GetInstance()->OnElementUpdated().Broadcast(UpdatedElements);
	}
}
#endif

TTypedElementOwner<FComponentElementData> UEngineElementsLibrary::CreateComponentElement(const UActorComponent* InComponent)
{
	TWeakObjectPtr<UActorComponent> ComponentWeak = const_cast<UActorComponent*>(InComponent);
	if (!ComponentWeak.IsValid())
	{
		return TTypedElementOwner<FComponentElementData>();
	}
	TTypedElementOwner<FComponentElementData> ComponentElement = EngineElementsLibraryUtil::CreateTypedElement<UActorComponent, FComponentElementData>(InComponent, NAME_Components);
	if (ComponentElement)
	{
		FComponentElementData& ElementData = ComponentElement.GetDataChecked();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ElementData.Component = const_cast<UActorComponent*>(InComponent);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		ElementData.ComponentWeak = const_cast<UActorComponent*>(InComponent);
	}
	return ComponentElement;
}

void UEngineElementsLibrary::DestroyComponentElement(const UActorComponent* InComponent, TTypedElementOwner<FComponentElementData>& InOutComponentElement)
{
	if (InOutComponentElement)
	{
		checkf(InOutComponentElement.GetDataChecked().ComponentWeak.GetEvenIfUnreachable() == InComponent, TEXT("Component element was not for this component instance! %s"), *InComponent->GetPathName());
		UTypedElementRegistry::GetInstance()->DestroyElement(InOutComponentElement);
	}
}

#if WITH_EDITOR
void UEngineElementsLibrary::CreateEditorComponentElement(const UActorComponent* Component)
{
	EngineElementsLibraryUtil::CreateEditorTypedElement<UActorComponent, FComponentElementData>(Component, GComponentElementOwnerStore, &UEngineElementsLibrary::CreateComponentElement);
}

void UEngineElementsLibrary::DestroyEditorComponentElement(const UActorComponent* Component)
{
	EngineElementsLibraryUtil::DestroyEditorTypedElement<UActorComponent, FComponentElementData>(Component, GComponentElementOwnerStore, &UEngineElementsLibrary::DestroyComponentElement);
}

FTypedElementHandle UEngineElementsLibrary::AcquireEditorComponentElementHandle(const UActorComponent* Component, const bool bAllowCreate)
{
	return EngineElementsLibraryUtil::AcquireEditorTypedElementHandle<UActorComponent, FComponentElementData>(Component, GComponentElementOwnerStore, &UEngineElementsLibrary::CreateComponentElement, bAllowCreate);
}

FScriptTypedElementHandle UEngineElementsLibrary::K2_AcquireEditorComponentElementHandle(const UActorComponent* Component, const bool bAllowCreate)
{
	if (!Component)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot acquire an editor component element handle from a nullptr component."), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	return UTypedElementRegistry::GetInstance()->CreateScriptHandle(AcquireEditorComponentElementHandle(Component, bAllowCreate).GetId());
}

void UEngineElementsLibrary::ReplaceEditorComponentElementHandles(const TMap<const UActorComponent*, const UActorComponent*>& ReplacementComponents)
{
	TArray<FTypedElementHandle> UpdatedElements;
	ReplaceEditorComponentElementHandlesImpl(ReplacementComponents, UpdatedElements);
	if (UpdatedElements.Num() > 0)
	{
		UTypedElementRegistry::GetInstance()->OnElementUpdated().Broadcast(UpdatedElements);
	}
}
#endif

TTypedElementOwner<FSMInstanceElementData> UEngineElementsLibrary::CreateSMInstanceElement(const FSMInstanceId& InSMInstanceId)
{
	const FSMInstanceElementId SMInstanceElementId = FSMInstanceElementIdMap::Get().GetSMInstanceElementIdFromSMInstanceId(InSMInstanceId);
	checkf(SMInstanceElementId, TEXT("Static Mesh Instance Index failed to map to a valid Static Mesh Instance Element ID!"));
	return CreateSMInstanceElementImpl(InSMInstanceId, SMInstanceElementId);
}

TTypedElementOwner<FSMInstanceElementData> UEngineElementsLibrary::CreateSMInstanceElementImpl(const FSMInstanceId& InSMInstanceId, const FSMInstanceElementId& InSMInstanceElementId)
{
	checkf(InSMInstanceId.ISMComponent == InSMInstanceElementId.ISMComponent, TEXT("Component mismatch in IDs!"));

	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	TTypedElementOwner<FSMInstanceElementData> SMInstanceElement;
	if (ensureAlways(Registry) && SMInstanceElementDataUtil::SMInstanceElementsEnabled() && SMInstanceElementDataUtil::GetSMInstanceManager(InSMInstanceId))
	{
		SMInstanceElement = Registry->CreateElement<FSMInstanceElementData>(NAME_SMInstance);
		if (SMInstanceElement)
		{
			SMInstanceElement.GetDataChecked().InstanceElementId = InSMInstanceElementId;
		}
	}

	return SMInstanceElement;
}

void UEngineElementsLibrary::DestroySMInstanceElement(const FSMInstanceElementId& InSMInstanceElementId, TTypedElementOwner<FSMInstanceElementData>& InOutSMInstanceElement)
{
	if (InOutSMInstanceElement)
	{
		checkf(InOutSMInstanceElement.GetDataChecked().InstanceElementId == InSMInstanceElementId, TEXT("Static Mesh Instance element was not for this instance! %s"), *InSMInstanceElementId.ISMComponent->GetPathName());
		UTypedElementRegistry::GetInstance()->DestroyElement(InOutSMInstanceElement);
	}
}

#if WITH_EDITOR
void UEngineElementsLibrary::CreateEditorSMInstanceElement(const FSMInstanceId& SMInstanceId)
{
	if (GIsEditor)
	{
		const FSMInstanceElementId SMInstanceElementId = FSMInstanceElementIdMap::Get().GetSMInstanceElementIdFromSMInstanceId(SMInstanceId);
		checkf(SMInstanceElementId, TEXT("Static Mesh Instance Index failed to map to a valid Static Mesh Instance Element ID!"));
		GSMInstanceElementOwnerStore.RegisterElementOwner(SMInstanceElementId, CreateSMInstanceElementImpl(SMInstanceId, SMInstanceElementId));
	}
}

void UEngineElementsLibrary::DestroyEditorSMInstanceElement(const FSMInstanceElementId& SMInstanceElementId)
{
	if (TTypedElementOwner<FSMInstanceElementData> SMInstanceElement = GSMInstanceElementOwnerStore.UnregisterElementOwner(SMInstanceElementId))
	{
		DestroySMInstanceElement(SMInstanceElementId, SMInstanceElement);
	}
}

FTypedElementHandle UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(const UInstancedStaticMeshComponent* ISMComponent, const int32 InstanceIndex, const bool bAllowCreate)
{
	return AcquireEditorSMInstanceElementHandle(FSMInstanceId{ const_cast<UInstancedStaticMeshComponent*>(ISMComponent), InstanceIndex }, bAllowCreate);
}

FScriptTypedElementHandle UEngineElementsLibrary::K2_AcquireEditorSMInstanceElementHandle(const UInstancedStaticMeshComponent* ISMComponent, const int32 InstanceIndex, const bool bAllowCreate)
{
	if (!ISMComponent)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot acquire an editor SMInstance element handle from a nullptr ISMComponent."), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	if (ISMComponent->GetInstanceCount() <= InstanceIndex || InstanceIndex < 0)
	{
		FStringFormatOrderedArguments FormatOrderedArguments;
		FormatOrderedArguments.Reserve(2);
		FormatOrderedArguments.Add(InstanceIndex);
		FormatOrderedArguments.Add(ISMComponent->GetInstanceCount());
		const FString Message = FString::Format(TEXT("Cannot acquire an editor SMInstance element handle from a instance that doesn't exist in the ISMComponent. (InstanceIndex: {0}, Instance Count: {1})"), FormatOrderedArguments);
		FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	return UTypedElementRegistry::GetInstance()->CreateScriptHandle(AcquireEditorSMInstanceElementHandle(ISMComponent, InstanceIndex, bAllowCreate).GetId());
}

FTypedElementHandle UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(const FSMInstanceId& SMInstanceId, const bool bAllowCreate)
{
	if (GIsEditor)
	{
		const FSMInstanceElementId SMInstanceElementId = FSMInstanceElementIdMap::Get().GetSMInstanceElementIdFromSMInstanceId(SMInstanceId, bAllowCreate);
		checkf(!bAllowCreate || SMInstanceElementId, TEXT("Static Mesh Instance Index failed to map to a valid Static Mesh Instance Element ID!"));
		
		TTypedElementOwnerScopedAccess<FSMInstanceElementData> SMInstanceElement = bAllowCreate
			? GSMInstanceElementOwnerStore.FindOrRegisterElementOwner(SMInstanceElementId, [&SMInstanceId, &SMInstanceElementId]() { return CreateSMInstanceElementImpl(SMInstanceId, SMInstanceElementId); })
			: GSMInstanceElementOwnerStore.FindElementOwner(SMInstanceElementId);

		if (SMInstanceElement)
		{
			return SMInstanceElement->AcquireHandle();
		}
	}

	return FTypedElementHandle();
}

FTypedElementHandle UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(const FSMInstanceElementId& SMInstanceElementId)
{
	if (GIsEditor)
	{
		TTypedElementOwnerScopedAccess<FSMInstanceElementData> SMInstanceElement = GSMInstanceElementOwnerStore.FindElementOwner(SMInstanceElementId);

		if (SMInstanceElement)
		{
			return SMInstanceElement->AcquireHandle();
		}
	}

	return FTypedElementHandle();
}

void UEngineElementsLibrary::ReplaceEditorSMInstanceElementHandles(const TMap<FSMInstanceId, FSMInstanceId>& ReplacementSMInstanceIds)
{
	TMap<FSMInstanceElementId, FSMInstanceElementId> PotentialSMInstanceReplacements;
	PotentialSMInstanceReplacements.Reserve(ReplacementSMInstanceIds.Num());

	FSMInstanceElementIdMap& SMInstanceElementIdMap = FSMInstanceElementIdMap::Get();
	for (const TTuple<FSMInstanceId, FSMInstanceId>& ReplacementSMInstanceIdPair : ReplacementSMInstanceIds)
	{
		// Find, but don't create, the element ID for the source instance, as no element can exist if the element ID has no mapping
		if (FSMInstanceElementId OldSMInstanceElementId = SMInstanceElementIdMap.GetSMInstanceElementIdFromSMInstanceId(ReplacementSMInstanceIdPair.Key, /*bAllowCreate*/false))
		{
			FSMInstanceElementId NewSMInstanceElementId = SMInstanceElementIdMap.GetSMInstanceElementIdFromSMInstanceId(ReplacementSMInstanceIdPair.Value, /*bAllowCreate*/true);
			PotentialSMInstanceReplacements.Add(MoveTemp(OldSMInstanceElementId), MoveTemp(NewSMInstanceElementId));
		}
	}

	TArray<FTypedElementHandle> UpdatedElements;
	ReplaceEditorSMInstanceElementHandlesImpl(PotentialSMInstanceReplacements, UpdatedElements);
	if (UpdatedElements.Num() > 0)
	{
		UTypedElementRegistry::GetInstance()->OnElementUpdated().Broadcast(UpdatedElements);
	}
}
#endif

