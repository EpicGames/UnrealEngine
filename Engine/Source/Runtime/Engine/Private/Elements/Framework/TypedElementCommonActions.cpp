// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementCommonActions.h"

#include "Algo/RemoveIf.h"
#include "Elements/Framework/TypedElementListProxy.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementUtil.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringOutputDevice.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementCommonActions)

namespace TypedElementCommonActionsUtils
{
	static int32 GEnableElementsCopyAndPaste = 0;
	static FAutoConsoleVariableRef CVarEnableElementsCopyAndPaste(
		TEXT("TypedElements.EnableElementsCopyAndPaste"),
		GEnableElementsCopyAndPaste,
		TEXT("Is support for elements copy and paste enabled?")
	);

	bool IsElementCopyAndPasteEnabled()
	{
		return GEnableElementsCopyAndPaste != 0;
	}
}
bool FTypedElementCommonActionsCustomization::DeleteElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	return InWorldInterface->DeleteElements(InElementHandles, InWorld, InSelectionSet, InDeletionOptions);
}

void FTypedElementCommonActionsCustomization::DuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
{
	InWorldInterface->DuplicateElements(InElementHandles, InWorld, InLocationOffset, OutNewElements);
}

bool FTypedElementCommonActionsCustomization::IsCopyCapable() const
{
	return false;
}

bool FTypedElementCommonActionsCustomization::CanCopyElements(ITypedElementWorldInterface* InWorldInterface,
                                                              TArrayView<const FTypedElementHandle> InElementHandles)
{
	// We can copy only if all are copyable AND there are handles that can be copied in the first place
	bool bCanCopy = !InElementHandles.IsEmpty();
	for (const FTypedElementHandle& Handle : InElementHandles)
	{
		if (!InWorldInterface->CanCopyElement(Handle))
		{
			return false;
		}
	}
	return bCanCopy;
}

bool FTypedElementCommonActionsCustomization::CanDuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles) const
{
	for (const FTypedElementHandle& Handle : InElementHandles)
	{
		if (!InWorldInterface->CanDuplicateElement(Handle))
		{
			return false;
		}
	}

	return true;
}

void FTypedElementCommonActionsCustomization::CopyElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out)
{
	InWorldInterface->CopyElements(InElementHandles, Out);
}

void FTypedElementCommonActionsCustomization::CutElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles,
	UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions,
	FOutputDevice& Out)
{
	CopyElements(InWorldInterface, InElementHandles, Out);
	DeleteElements(InWorldInterface, InElementHandles, InWorld, InSelectionSet, InDeletionOptions);
}

TSharedPtr<FWorldElementPasteImporter> FTypedElementCommonActionsCustomization::GetPasteImporter(ITypedElementWorldInterface* InWorldInterface, const FTypedElementListConstPtr& InSelectedHandles, UWorld* InWorld)
{
	return InWorldInterface->GetPasteImporter();
}

bool UTypedElementCommonActions::DeleteSelectedElements(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementDeletionOptions& DeletionOptions)
{
	if (!SelectionSet)
	{
		FFrame::KismetExecutionMessage(TEXT("SelectionSet is null."), ELogVerbosity::Error);
		return false;
	}

	if (!World)
	{
		FFrame::KismetExecutionMessage(TEXT("World is null."), ELogVerbosity::Error);
		return false;
	}

	FTypedElementListRef NormalizedElements = SelectionSet->GetNormalizedSelection(FTypedElementSelectionNormalizationOptions());
	return DeleteNormalizedElements(NormalizedElements, World, SelectionSet, DeletionOptions);
}

bool UTypedElementCommonActions::DeleteNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, UTypedElementSelectionSet* SelectionSet, const FTypedElementDeletionOptions& DeletionOptions)
{
	bool bSuccess = false;

	if (ElementListPtr)
	{
		TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>> ElementsToDeleteByType;
		TypedElementUtil::BatchElementsByType(ElementListPtr.ToSharedRef(), ElementsToDeleteByType);

		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		UTypedElementRegistry::FDisableElementDestructionOnGC GCGuard(Registry);

		for (const auto& ElementsByTypePair : ElementsToDeleteByType)
		{
			FTypedElementCommonActionsCustomization* CommonActionsCustomization = GetInterfaceCustomizationByTypeId(ElementsByTypePair.Key);
			ITypedElementWorldInterface* WorldInterface = Registry->GetElementInterface<ITypedElementWorldInterface>(ElementsByTypePair.Key);
			if (CommonActionsCustomization && WorldInterface)
			{
				bSuccess |= CommonActionsCustomization->DeleteElements(WorldInterface, ElementsByTypePair.Value, World, SelectionSet, DeletionOptions);
			}
		}
	}
	
	return bSuccess;
}

TArray<FTypedElementHandle> UTypedElementCommonActions::DuplicateSelectedElements(const UTypedElementSelectionSet* SelectionSet, UWorld* World, const FVector& LocationOffset)
{
	FTypedElementListRef NormalizedElements = SelectionSet->GetNormalizedSelection(FTypedElementSelectionNormalizationOptions().SetNameForTEDSIntegration(SelectionSet->GetNameForTedsIntegration()));
	return DuplicateNormalizedElements(NormalizedElements, World, LocationOffset);
}

bool UTypedElementCommonActions::CanDuplicateSelectedElements(const UTypedElementSelectionSet* SelectionSet)
{
	if (SelectionSet)
	{
		if (SelectionSet->GetNumSelectedElements() == 0)
		{
			//  return true for legacy duplication code in mode managers to operate on types that may not be hooked into typed elements yet
			return true;
		}
		TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>> ElementsToDuplicateByType;
		TypedElementUtil::BatchElementsByType(SelectionSet->GetNormalizedSelection(FTypedElementSelectionNormalizationOptions().SetNameForTEDSIntegration(SelectionSet->GetNameForTedsIntegration())), ElementsToDuplicateByType);

		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		for (const auto& ElementsByTypePair : ElementsToDuplicateByType)
		{
			FTypedElementCommonActionsCustomization* CommonActionsCustomization = GetInterfaceCustomizationByTypeId(ElementsByTypePair.Key);
			ITypedElementWorldInterface* WorldInterface = Registry->GetElementInterface<ITypedElementWorldInterface>(ElementsByTypePair.Key);
			if (CommonActionsCustomization && WorldInterface)
			{
				if (!CommonActionsCustomization->CanDuplicateElements(WorldInterface, ElementsByTypePair.Value))
				{
					return false;
				}
			}
		}
	}

	return true;
}

TArray<FTypedElementHandle> UTypedElementCommonActions::DuplicateNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, const FVector& LocationOffset)
{
	TArray<FTypedElementHandle> NewElements;
	if (ElementListPtr)
	{
		NewElements.Reserve(ElementListPtr->Num());

		TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>> ElementsToDuplicateByType;
		TypedElementUtil::BatchElementsByType(ElementListPtr.ToSharedRef(), ElementsToDuplicateByType);

		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		for (const auto& ElementsByTypePair : ElementsToDuplicateByType)
		{
			FTypedElementCommonActionsCustomization* CommonActionsCustomization = GetInterfaceCustomizationByTypeId(ElementsByTypePair.Key);
			ITypedElementWorldInterface* WorldInterface = Registry->GetElementInterface<ITypedElementWorldInterface>(ElementsByTypePair.Key);
			if (CommonActionsCustomization && WorldInterface)
			{
				CommonActionsCustomization->DuplicateElements(WorldInterface, ElementsByTypePair.Value, World, LocationOffset, NewElements);
			}
		}
	}
	
	return NewElements;
}

FTypedElementCommonActions_CopyPasteCapability UTypedElementCommonActions::CanCopySelectedElements(const UTypedElementSelectionSet* SelectionSet)
{
	if (SelectionSet == nullptr)
	{
		return FTypedElementCommonActions_CopyPasteCapability();
	}
	
	FTypedElementListRef NormalizedElements = SelectionSet->GetNormalizedSelection(FTypedElementSelectionNormalizationOptions());
	
	TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>> ElementsToCopyByType;
	TypedElementUtil::BatchElementsByType(NormalizedElements, ElementsToCopyByType);
	
	if (ElementsToCopyByType.IsEmpty())
	{
		return FTypedElementCommonActions_CopyPasteCapability();
	}

	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	
	bool bCapable = true;
	bool bAllowed = true;
	for (const TPair<FTypedHandleTypeId, TArray<FTypedElementHandle>>& ElementsByTypePair : ElementsToCopyByType)
	{
		FTypedElementCommonActionsCustomization* CommonActionsCustomization = GetInterfaceCustomizationByTypeId(ElementsByTypePair.Key);
		ITypedElementWorldInterface* WorldInterface = Registry->GetElementInterface<ITypedElementWorldInterface>(ElementsByTypePair.Key);
		if (CommonActionsCustomization && WorldInterface)
		{
			bCapable = bCapable && CommonActionsCustomization->IsCopyCapable();
			bAllowed = bAllowed && CommonActionsCustomization->CanCopyElements(WorldInterface, ElementsByTypePair.Value);
		}
		else
		{
			bCapable = false;
			bAllowed = false;
			break; // No need to check more
		}
	}
	return FTypedElementCommonActions_CopyPasteCapability
	{
		.bCapable = bCapable,
		.bAllowed = bAllowed,
	};
}

bool UTypedElementCommonActions::CopySelectedElements(UTypedElementSelectionSet* SelectionSet)
{
	if (!SelectionSet)
	{
		FFrame::KismetExecutionMessage(TEXT("SelectionSet is null."), ELogVerbosity::Error);
		return false;
	}

	FTypedElementListRef NormalizedElements = SelectionSet->GetNormalizedSelection(FTypedElementSelectionNormalizationOptions());
	return CopyNormalizedElements(NormalizedElements);
}

bool UTypedElementCommonActions::CopySelectedElementsToString(UTypedElementSelectionSet* SelectionSet, FString& OutputString)
{
	if (!SelectionSet)
	{
		FFrame::KismetExecutionMessage(TEXT("SelectionSet is null."), ELogVerbosity::Error);
		return false;
	}

	FTypedElementListRef NormalizedElements = SelectionSet->GetNormalizedSelection(FTypedElementSelectionNormalizationOptions());
	return CopyNormalizedElements(NormalizedElements, &OutputString);
}

bool UTypedElementCommonActions::CopyNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, FString* OptionalOutputString)
{
	return CutOrCopyNormalizedElements(ElementListPtr, OptionalOutputString, []
		(FTypedElementCommonActionsCustomization& CommonActionsCustomization, ITypedElementWorldInterface& WorldInterface, 
			const TArray<FTypedElementHandle>& ElementsIn, FStringOutputDevice& Output)
		{
			CommonActionsCustomization.CopyElements(&WorldInterface, ElementsIn, Output);
		});
}

bool UTypedElementCommonActions::CutOrCopyNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, FString* OptionalOutputString,
	TFunctionRef<void(FTypedElementCommonActionsCustomization& CommonActionsCustomization, ITypedElementWorldInterface& WorldInterface, 
		const TArray<FTypedElementHandle>& ElementsIn, FStringOutputDevice& Output)> CutOrCopy)
{
	if (!ElementListPtr)
	{
		return false;
	}
	
	FStringOutputDevice Out;

	TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>> ElementsToCopyByType;
	TypedElementUtil::BatchElementsByType(ElementListPtr.ToSharedRef(), ElementsToCopyByType);

	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	for (const TPair<FTypedHandleTypeId, TArray<FTypedElementHandle>>& ElementsByTypePair : ElementsToCopyByType)
	{
		// Store the copied elements in a separate string to test if anything was copied before writing element group
		FStringOutputDevice ElementsOut;
		
		FTypedElementCommonActionsCustomization* CommonActionsCustomization = GetInterfaceCustomizationByTypeId(ElementsByTypePair.Key);
		ITypedElementWorldInterface* WorldInterface = Registry->GetElementInterface<ITypedElementWorldInterface>(ElementsByTypePair.Key);
		if (CommonActionsCustomization && WorldInterface)
		{
			CutOrCopy(*CommonActionsCustomization, *WorldInterface, ElementsByTypePair.Value, ElementsOut);
		}
		if (ElementsOut.IsEmpty())
		{
			continue;
		}
		
		FName ElementName = Registry->GetRegisteredElementTypeName(ElementsByTypePair.Key);
		Out.Logf(TEXT("Begin ElementGroup Type=%s") LINE_TERMINATOR, *ElementName.ToString());
		Out.Append(ElementsOut);
		Out.Logf(TEXT("End ElementGroup") LINE_TERMINATOR);
	}

	bool bHasExportedSomething = !Out.IsEmpty();
	
	if (OptionalOutputString)
	{
		*OptionalOutputString = MoveTemp(Out);
	}
	else
	{
		FPlatformApplicationMisc::ClipboardCopy(*Out);
	}

	return bHasExportedSomething;
}

bool UTypedElementCommonActions::CutSelectedElements(UTypedElementSelectionSet* SelectionSet, UWorld* World, 
	const FTypedElementDeletionOptions& DeletionOptions, FString* OptionalOutputString)
{
	if (!SelectionSet || !World)
	{
		return false;
	}

	FTypedElementListRef NormalizedElements = SelectionSet->GetNormalizedSelection(FTypedElementSelectionNormalizationOptions());
	return CutNormalizedElements(NormalizedElements, World, SelectionSet, DeletionOptions, OptionalOutputString);
}

bool UTypedElementCommonActions::CutNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, 
	UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& DeletionOptions, FString* OptionalOutputString)
{
	return CutOrCopyNormalizedElements(ElementListPtr, OptionalOutputString, [World, InSelectionSet, &DeletionOptions]
		(FTypedElementCommonActionsCustomization& CommonActionsCustomization, ITypedElementWorldInterface& WorldInterface, 
			const TArray<FTypedElementHandle>& ElementsIn, FStringOutputDevice& Output)
		{
			CommonActionsCustomization.CutElements(&WorldInterface, ElementsIn, World, InSelectionSet, DeletionOptions, Output);
		});
}

TArray<TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>> UTypedElementCommonActions::ParsePasteTextAndCreateImporters(
	const FTypedElementPasteOptions& PasteOptions, 
	const FString& InputString, 
	UTypedElementRegistry* Registry, 
	const FTypedElementListConstPtr& ElementListPtr, 
	UWorld* World)
{
	const TCHAR* StringToImport = *InputString;
	
	TArray<TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>> ImportersAndText;
	
	// Break the text into the group for their respective importer
	const TCHAR** CurrentPosInString = &StringToImport;

	FStringView* CurrentBlock = nullptr;
	const TCHAR* StartOfBlock = nullptr;
	FString LookingForEndTag;
	FStringView Line;
	const TCHAR* BeginChars = TEXT("Begin ");
	const int32 BeginCharsLen = FCString::Strlen(BeginChars);
	TCHAR Buffer[NAME_SIZE];
	const TCHAR* TypedElementChar = TEXT("ElementGroup");

	while (FParse::Line(CurrentPosInString, Line))
	{
		if (!LookingForEndTag.IsEmpty())
		{
			if (Line.StartsWith(LookingForEndTag))
			{
				if (CurrentBlock)
				{
					(*CurrentBlock) = MakeStringView(StartOfBlock, int32((Line.GetData() + Line.Len()) - StartOfBlock));
				}
				LookingForEndTag.Reset();
			}
		}
		else if (Line.StartsWith(BeginChars))
		{
			const TCHAR* PositionInLine = Line.GetData() + BeginCharsLen;
			if (FParse::Token(PositionInLine, Buffer, NAME_SIZE, false))
			{
				if (FCString::Strcmp(TypedElementChar, Buffer) == 0)
				{
					// Trim any spaces at the start of the line
					while(PositionInLine[0] == TEXT(' '))
					{
						++PositionInLine;
					}

					FName TypeName;
					if (!FParse::Value(PositionInLine, TEXT("Type="), TypeName))
					{
						continue;
					}
					FTypedHandleTypeId TypeID = Registry->GetRegisteredElementTypeId(TypeName);
					if (TypeID == 0)
					{
						continue;
					}
					ITypedElementWorldInterface* WorldInterface = Registry->GetElementInterface<ITypedElementWorldInterface>(TypeID);
					if (!WorldInterface)
					{
						continue;
					}
					FTypedElementCommonActionsCustomization* CommonActionsCustomization = GetInterfaceCustomizationByTypeId(TypeID);
					if (CommonActionsCustomization == nullptr)
					{
						// Ignore blocks for which we cannot find a common action set
						continue;
					}
					TSharedPtr<FWorldElementPasteImporter> Importer = CommonActionsCustomization->GetPasteImporter(WorldInterface, ElementListPtr, World);
					if (!Importer)
					{
						continue;
					}
					StartOfBlock = Line.GetData();
					TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>& Pair = ImportersAndText.Add_GetRef(
						TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>(Importer.ToSharedRef(), FStringView())
						);
					CurrentBlock = &Pair.Value;

					LookingForEndTag = FString::Printf(TEXT("End %s"), Buffer);
				}
				else
				{
					FStringView BufferView(Buffer);
					if (const TFunction<TSharedRef<FWorldElementPasteImporter> ()>* GetImporterFunction = PasteOptions.ExtraCustomImport.Find(BufferView))
					{
						StartOfBlock = Line.GetData();
						TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>& Pair = ImportersAndText.Add_GetRef(
							TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>((*GetImporterFunction)(), FStringView())
						);
						CurrentBlock = &Pair.Value;

						LookingForEndTag = FString::Printf(TEXT("End %s"), Buffer);
					}
				}
			}
		}
	}
	
	// Remove importers with empty inner text
	const int32 LastInRange = Algo::StableRemoveIf(ImportersAndText, [](TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>& Pair)
		{
			return Pair.Value.IsEmpty();
		});
	ImportersAndText.SetNum(LastInRange);
	
	return ImportersAndText;
}

FTypedElementCommonActions_CopyPasteCapability UTypedElementCommonActions::CanPasteElements(
	const FTypedElementPasteOptions& PasteOptions,
	const FString* OptionalInputString)
{
	FString ClipboardString;
	if (!OptionalInputString)
	{
		FPlatformApplicationMisc::ClipboardPaste(ClipboardString);
	}
	
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	TArray<TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>> ImportersAndText;
	
	const FTypedElementListConstPtr ElementListPtr;
	UWorld* World = nullptr;
	
	ImportersAndText = ParsePasteTextAndCreateImporters(
		PasteOptions, 
		OptionalInputString ? *OptionalInputString : ClipboardString, 
		Registry,
		ElementListPtr,
		World);
	
	// If there is at least one Begin <Type> that has a paste importer assigned to it
	// then it is possible to paste the elements
	// Note, this does mean that ElementGroups where there is no paste importer will be skipped during the paste
	// If it is desired to not skip them, then implement the paste importer for the respective type
	// Also note, permissibility differs depending on copy (where the output is controlled) and paste (where the input is not).
	// In the copy case, we require everything to be copyable.  In the paste case, it is best effort and paste as much as we are capable.
	if (ImportersAndText.IsEmpty())
	{
		return FTypedElementCommonActions_CopyPasteCapability
		{
			.bCapable = false,
			.bAllowed = false
		};
	}
	else
	{
		return FTypedElementCommonActions_CopyPasteCapability
		{
			.bCapable = true,
			.bAllowed = true
		};
	}
}

TArray<FTypedElementHandle> UTypedElementCommonActions::PasteElements(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementPasteOptions& PasteOption, const FString* OptionalInputString)
{
	FTypedElementListRef NormalizedElements = SelectionSet->GetNormalizedSelection(FTypedElementSelectionNormalizationOptions());
	return PasteNormalizedElements(NormalizedElements, World, PasteOption, OptionalInputString);
}

TArray<FTypedElementHandle> UTypedElementCommonActions::PasteNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, const FTypedElementPasteOptions& PasteOptions, const FString* OptionalInputString)
{
	check(World)

	FString ClipboardString;
	const TCHAR* StringToImport;
	if (OptionalInputString)
	{
		StringToImport = **OptionalInputString;
	}
	else
	{
		FPlatformApplicationMisc::ClipboardPaste(ClipboardString);
		StringToImport = *ClipboardString;
	}

	UTypedElementRegistry* Registry = ElementListPtr->GetRegistry();
	TArray<TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>> ImportersAndText;
	
	ImportersAndText = ParsePasteTextAndCreateImporters(
		PasteOptions, 
		OptionalInputString ? *OptionalInputString : ClipboardString, 
		Registry,
		ElementListPtr,
		World);

	TArray<FTypedElementHandle> PastedHandles; 

	// Import the elements
	{
		TMap<FString, UObject*> ExportedPathToObjectCreated;
		for (TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>& Pair : ImportersAndText)
		{
			FWorldElementPasteImporter::FContext Context;
			Context.CurrentSelection = ElementListPtr;
			Context.World = World;
			Context.Text = Pair.Value;
			Context.PasteUnder = PasteOptions.PasteUnder.Get(FTypedElementHandle());
			Pair.Key->Import(Context);
		}
		
		for (TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>& Pair : ImportersAndText)
		{
			PastedHandles.Append(Pair.Key->GetImportedElements());
		}
	}

	// Paste at implementation
	if (PasteOptions.bPasteAtLocation)
	{
		FBox BoundingBox(ForceInit);

		TArray<TTypedElement<ITypedElementWorldInterface>> WorldElements;
		WorldElements.Reserve(PastedHandles.Num());
		for (const FTypedElementHandle& Handle : PastedHandles)
		{
			if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(Handle))
			{
				FTransform WorldTransform;
				WorldElement.GetWorldTransform(WorldTransform);
				BoundingBox += WorldTransform.GetLocation();

				WorldElements.Add(MoveTemp(WorldElement));
			}
		}

		FVector Offset = PasteOptions.PasteLocation - BoundingBox.GetCenter();
		for (TTypedElement<ITypedElementWorldInterface> WorldElement : WorldElements)
		{
			FTransform WorldTransform;
			WorldElement.GetWorldTransform(WorldTransform);
			WorldTransform.SetLocation(WorldTransform.GetLocation() + Offset);
			WorldElement.SetWorldTransform(WorldTransform);
		}
	}

	// Set the selection optionally
	if (PasteOptions.SelectionSetToModify && !PastedHandles.IsEmpty())
	{
		PasteOptions.SelectionSetToModify->SetSelection(PastedHandles, FTypedElementSelectionOptions());
	}

	return PastedHandles;
}

bool UTypedElementCommonActions::DeleteNormalizedElements(const FScriptTypedElementListProxy ElementList, UWorld* World, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& DeletionOptions)
{
	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("ElementList is in a invalid state."), ELogVerbosity::Error);
		return false;
	}

	if (!World)
	{
		FFrame::KismetExecutionMessage(TEXT("World is null."), ELogVerbosity::Error);
		return false;
	}

	return DeleteNormalizedElements(NativeList, World, InSelectionSet, DeletionOptions);
}

TArray<FScriptTypedElementHandle> UTypedElementCommonActions::K2_DuplicateSelectedElements(const UTypedElementSelectionSet* SelectionSet, UWorld* World, const FVector& LocationOffset)
{
	if (!SelectionSet)
	{
		FFrame::KismetExecutionMessage(TEXT("SelectionSet is null."), ELogVerbosity::Error);
		return {};
	}

	if (!World)
	{
		FFrame::KismetExecutionMessage(TEXT("World is null."), ELogVerbosity::Error);
		return {};
	}

	return TypedElementUtil::ConvertToScriptElementArray(DuplicateSelectedElements(SelectionSet, World, LocationOffset), SelectionSet->GetElementList()->GetRegistry());
}

TArray<FScriptTypedElementHandle> UTypedElementCommonActions::DuplicateNormalizedElements(const FScriptTypedElementListProxy ElementList, UWorld* World, const FVector& LocationOffset)
{
	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("ElementList is in a invalid state."), ELogVerbosity::Error);
		return {};
	}

	if (!World)
	{
		FFrame::KismetExecutionMessage(TEXT("World is null."), ELogVerbosity::Error);
		return {};
	}

	return TypedElementUtil::ConvertToScriptElementArray(DuplicateNormalizedElements(NativeList, World, LocationOffset), NativeList->GetRegistry());
}

bool UTypedElementCommonActions::CopyNormalizedElements(const FScriptTypedElementListProxy& ElementList)
{
	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("ElementList is in a invalid state."), ELogVerbosity::Error);
		return {};
	}

	return CopyNormalizedElements(NativeList);
}

bool UTypedElementCommonActions::CopyNormalizedElementsToString(const FScriptTypedElementListProxy& ElementList, FString& OutputString)
{
	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("ElementList is in a invalid state."), ELogVerbosity::Error);
		return {};
	}

	return CopyNormalizedElements(NativeList, &OutputString);
}

TArray<FScriptTypedElementHandle> UTypedElementCommonActions::K2_PasteElements(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementPasteOptions& PasteOption)
{
	if (!SelectionSet)
	{
		FFrame::KismetExecutionMessage(TEXT("SelectionSet is null."), ELogVerbosity::Error);
		return {};
	}

	if (!World)
	{
		FFrame::KismetExecutionMessage(TEXT("World is null."), ELogVerbosity::Error);
		return {};
	}

	return TypedElementUtil::ConvertToScriptElementArray(PasteElements(SelectionSet, World, PasteOption), SelectionSet->GetElementList()->GetRegistry());
}

TArray<FScriptTypedElementHandle> UTypedElementCommonActions::PasteElementsFromString(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementPasteOptions& PasteOption, const FString& InputString)
{
	if (!SelectionSet)
	{
		FFrame::KismetExecutionMessage(TEXT("SelectionSet is null."), ELogVerbosity::Error);
		return {};
	}

	if (!World)
	{
		FFrame::KismetExecutionMessage(TEXT("World is null."), ELogVerbosity::Error);
		return {};
	}

	return TypedElementUtil::ConvertToScriptElementArray(PasteElements(SelectionSet, World, PasteOption, &InputString), SelectionSet->GetElementList()->GetRegistry());
}

TArray<FScriptTypedElementHandle> UTypedElementCommonActions::K2_PasteNormalizedElements(const FScriptTypedElementListProxy& ElementList, UWorld* World, const FTypedElementPasteOptions& PasteOption)
{
	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("ElementList is in a invalid state."), ELogVerbosity::Error);
		return {};
	}

	if (!World)
	{
		FFrame::KismetExecutionMessage(TEXT("World is null."), ELogVerbosity::Error);
		return {};
	}

	return TypedElementUtil::ConvertToScriptElementArray(PasteNormalizedElements(NativeList, World, PasteOption), NativeList->GetRegistry());
}

TArray<FScriptTypedElementHandle> UTypedElementCommonActions::PasteNormalizedElementsFromString(const FScriptTypedElementListProxy& ElementList, UWorld* World, const FTypedElementPasteOptions& PasteOption, const FString& InputString)
{
	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("ElementList is in a invalid state."), ELogVerbosity::Error);
		return {};
	}

	if (!World)
	{
		FFrame::KismetExecutionMessage(TEXT("World is null."), ELogVerbosity::Error);
		return {};
	}

	return TypedElementUtil::ConvertToScriptElementArray(PasteNormalizedElements(NativeList, World, PasteOption, &InputString), NativeList->GetRegistry());
}


FTypedElementCommonActionsElement UTypedElementCommonActions::ResolveCommonActionsElement(const FTypedElementHandle& InElementHandle) const
{
	return InElementHandle
		? FTypedElementCommonActionsElement(UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(InElementHandle), GetInterfaceCustomizationByTypeId(InElementHandle.GetId().GetTypeId()))
		: FTypedElementCommonActionsElement();
}

