// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementWorldInterface.h"

#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementHierarchyInterface.h"

#include "ConvexVolume.h"
#include "Math/BoxSphereBounds.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementWorldInterface)

bool ITypedElementWorldInterface::IsElementInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, bool bMustEncompassEntireElement)
{
	FBoxSphereBounds Bounds;
	if (GetBounds(Handle, Bounds))
	{
		bool bIsFullyContained = false;
		if (InVolume.IntersectBox(Bounds.Origin, Bounds.BoxExtent, bIsFullyContained))
		{
			return !bMustEncompassEntireElement || bIsFullyContained;
		}
	}

	return false;
}

bool ITypedElementWorldInterface::IsElementInBox(const FTypedElementHandle& Handle, const FBox& InBox, bool bMustEncompassEntireElement)
{
	FBoxSphereBounds Bounds;
	if (GetBounds(Handle, Bounds))
	{
		const FBox& ElementBounds = Bounds.GetBox();

		// Check the component bounds versus the selection box
		// If the selection box must encompass the entire component, then both the min and max vector of the bounds must be inside in the selection
		// box to be valid. If the selection box only has to touch the component, then it is sufficient to check if it intersects with the bounds.
		if ((!bMustEncompassEntireElement && InBox.Intersect(ElementBounds))
			|| (bMustEncompassEntireElement && InBox.IsInside(ElementBounds)))
		{
			return true;
		}
	}

	return false;
}

TArray<FTypedElementHandle> ITypedElementWorldInterface::GetSelectionElementsInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, const FWorldSelectionElementArgs& SelectionArgs)
{
	UTypedElementRegistry& Registry = GetRegistry();
	return GetSelectionElementsFromSelectionFunction(Handle, SelectionArgs, [&InVolume, &Registry](const FTypedElementHandle& ElementHandle, const FWorldSelectionElementArgs& SelectionArgs) -> bool
	{
		if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry.GetElement<ITypedElementWorldInterface>(ElementHandle))
		{
			return WorldElement.IsElementInConvexVolume(InVolume, SelectionArgs.bMustEncompassEntireElement);
		}

		return false;
	});
}

TArray<FTypedElementHandle> ITypedElementWorldInterface::GetSelectionElementsInBox(const FTypedElementHandle& Handle, const FBox& InBox, const FWorldSelectionElementArgs& SelectionArgs)
{
	UTypedElementRegistry& Registry = GetRegistry();
	return GetSelectionElementsFromSelectionFunction(Handle, SelectionArgs, [&InBox, &Registry](const FTypedElementHandle& ElementHandle, const FWorldSelectionElementArgs& SelectionArgs) -> bool
	{
		if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry.GetElement<ITypedElementWorldInterface>(ElementHandle))
		{
			return WorldElement.IsElementInBox(InBox, SelectionArgs.bMustEncompassEntireElement);
		}

		return false;
	});
}

TArray<FTypedElementHandle> ITypedElementWorldInterface::GetSelectionElementsFromSelectionFunction(const FTypedElementHandle& InElementHandle, const FWorldSelectionElementArgs& SelectionArgs, const TFunction<bool(const FTypedElementHandle& /*ElementHandle*/, const FWorldSelectionElementArgs& /*SelectionArgs*/)>& SelectionFunction)
{
	// The default implementation simply iterate the child elements

	TArray<FTypedElementHandle> SelectedSubElements;
	bool bTryToSelectCurrent = true;
	if (TTypedElement<ITypedElementHierarchyInterface> HierarchyElement = GetRegistry().GetElement<ITypedElementHierarchyInterface>(InElementHandle))
	{
		TArray<FTypedElementHandle> ChildElementHandles;
		HierarchyElement.GetChildElements(ChildElementHandles);

		bTryToSelectCurrent = ChildElementHandles.IsEmpty();

		for (const FTypedElementHandle& ChildHandle : ChildElementHandles)
		{
			if (TTypedElement<ITypedElementWorldInterface> ChildWorldElement = GetRegistry().GetElement<ITypedElementWorldInterface>(ChildHandle))
			{
				TArray<FTypedElementHandle> ElementsFromChildSelection = ChildWorldElement.GetSelectionElementsFromSelectionFunction(SelectionArgs, SelectionFunction);
				if (ElementsFromChildSelection.Num() > 1)
				{
					SelectedSubElements.Reserve(SelectedSubElements.Num() + ElementsFromChildSelection.Num());
				}

				for (const FTypedElementHandle& Handle : ElementsFromChildSelection)
				{
					if (Handle == InElementHandle)
					{
						return { InElementHandle };
					}

					SelectedSubElements.Add(Handle);
				}
			}
		}
	
		if (!bTryToSelectCurrent && ChildElementHandles == SelectedSubElements)
		{
			// If all the elements selected are the child elements, add the current also if possible.
			if (SelectionArgs.SelectionSet)
			{
				SelectedSubElements.Add(SelectionArgs.SelectionSet->GetSelectionElement(InElementHandle, SelectionArgs.SelectionMethod));
			}
			else
			{
				SelectedSubElements.Add(InElementHandle);
			}
		}
	}
	
	if (bTryToSelectCurrent && SelectionFunction(InElementHandle, SelectionArgs))
	{
		if (SelectionArgs.SelectionSet)
		{
			SelectedSubElements.Add(SelectionArgs.SelectionSet->GetSelectionElement(InElementHandle, SelectionArgs.SelectionMethod));
		}
		else
		{
			SelectedSubElements.Add(InElementHandle);
		}
	}

	return SelectedSubElements;
}

bool ITypedElementWorldInterface::IsTemplateElement(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return IsTemplateElement(NativeHandle);
}

bool ITypedElementWorldInterface::CanEditElement(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanEditElement(NativeHandle);
}

ULevel* ITypedElementWorldInterface::GetOwnerLevel(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return nullptr;
	}

	return GetOwnerLevel(NativeHandle);
}

UWorld* ITypedElementWorldInterface::GetOwnerWorld(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return nullptr;
	}

	return GetOwnerWorld(NativeHandle);
}

bool ITypedElementWorldInterface::GetBounds(const FScriptTypedElementHandle& InElementHandle, FBoxSphereBounds& OutBounds)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return GetBounds(NativeHandle, OutBounds);
}

bool ITypedElementWorldInterface::CanMoveElement(const FScriptTypedElementHandle& InElementHandle, const ETypedElementWorldType InWorldType)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanMoveElement(NativeHandle, InWorldType);
}

bool ITypedElementWorldInterface::CanScaleElement(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanScaleElement(NativeHandle);
}

bool ITypedElementWorldInterface::GetWorldTransform(const FScriptTypedElementHandle& InElementHandle, FTransform& OutTransform)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return GetWorldTransform(NativeHandle, OutTransform);
}

bool ITypedElementWorldInterface::SetWorldTransform(const FScriptTypedElementHandle& InElementHandle, const FTransform& InTransform)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return SetWorldTransform(NativeHandle, InTransform);
}

bool ITypedElementWorldInterface::GetRelativeTransform(const FScriptTypedElementHandle& InElementHandle, FTransform& OutTransform)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return GetRelativeTransform(NativeHandle, OutTransform);
}

bool ITypedElementWorldInterface::SetRelativeTransform(const FScriptTypedElementHandle& InElementHandle, const FTransform& InTransform)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return SetRelativeTransform(NativeHandle, InTransform);
}

bool ITypedElementWorldInterface::GetPivotOffset(const FScriptTypedElementHandle& InElementHandle, FVector& OutPivotOffset)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return GetPivotOffset(NativeHandle, OutPivotOffset);
}

bool ITypedElementWorldInterface::SetPivotOffset(const FScriptTypedElementHandle& InElementHandle, const FVector& InPivotOffset)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return SetPivotOffset(NativeHandle, InPivotOffset);
}


void ITypedElementWorldInterface::NotifyMovementStarted(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return;
	}

	NotifyMovementStarted(NativeHandle);
}


void ITypedElementWorldInterface::NotifyMovementOngoing(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return;
	}

	NotifyMovementOngoing(NativeHandle);
}

void ITypedElementWorldInterface::NotifyMovementEnded(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return;
	}

	NotifyMovementEnded(NativeHandle);
}

bool ITypedElementWorldInterface::CanDeleteElement(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanDeleteElement(NativeHandle);
}

bool ITypedElementWorldInterface::DeleteElement(const FScriptTypedElementHandle& InElementHandle, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	if (!InWorld)
	{
		FFrame::KismetExecutionMessage(TEXT("InWorld is null"), ELogVerbosity::Error);
		return false;
	}

	if (!InSelectionSet)
	{
		FFrame::KismetExecutionMessage(TEXT("InSelectionSet is null"), ELogVerbosity::Error);
		return false;
	}

	return DeleteElement(NativeHandle, InWorld, InSelectionSet, InDeletionOptions);
}

bool ITypedElementWorldInterface::CanDuplicateElement(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanDuplicateElement(NativeHandle);
}

FScriptTypedElementHandle ITypedElementWorldInterface::DuplicateElement(const FScriptTypedElementHandle& InElementHandle, UWorld* InWorld, const FVector& InLocationOffset)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	if (!InWorld)
	{
		FFrame::KismetExecutionMessage(TEXT("InWorld is null"), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	return GetRegistry().CreateScriptHandle(DuplicateElement(NativeHandle, InWorld, InLocationOffset).GetId());
}

bool ITypedElementWorldInterface::CanPromoteElement(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanPromoteElement(NativeHandle);
}

FScriptTypedElementHandle ITypedElementWorldInterface::PromoteElement(const FScriptTypedElementHandle& InElementHandle, UWorld* OverrideWorld)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	return GetRegistry().CreateScriptHandle(PromoteElement(NativeHandle, OverrideWorld).GetId());
}

class UTypedElementRegistry& ITypedElementWorldInterface::GetRegistry() const
{
	return *UTypedElementRegistry::GetInstance();
}

