// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Object/ObjectElementAssetDataInterface.h"

#include "Elements/Object/ObjectElementData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectElementAssetDataInterface)

FAssetData UObjectElementAssetDataInterface::GetAssetData(const FTypedElementHandle& InElementHandle)
{
	UObject* RawObjectPtr = ObjectElementDataUtil::GetObjectFromHandle(InElementHandle);
	return FAssetData(RawObjectPtr);
}

