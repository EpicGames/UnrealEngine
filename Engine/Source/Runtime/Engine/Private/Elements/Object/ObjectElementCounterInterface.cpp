// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Object/ObjectElementCounterInterface.h"
#include "Elements/Object/ObjectElementData.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectElementCounterInterface)

void UObjectElementCounterInterface::IncrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter)
{
	if (const UObject* Object = ObjectElementDataUtil::GetObjectFromHandle(InElementHandle))
	{
		UObjectElementCounterInterface::IncrementCounterForObjectClass(Object, InOutCounter);
	}
}

void UObjectElementCounterInterface::DecrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter)
{
	if (const UObject* Object = ObjectElementDataUtil::GetObjectFromHandle(InElementHandle))
	{
		UObjectElementCounterInterface::DecrementCounterForObjectClass(Object, InOutCounter);
	}
}

void UObjectElementCounterInterface::IncrementCounterForObjectClass(const UObject* InObject, FTypedElementCounter& InOutCounter)
{
	check(InObject);
	InOutCounter.IncrementCounter(NAME_Class, InObject->GetClass());
}

void UObjectElementCounterInterface::DecrementCounterForObjectClass(const UObject* InObject, FTypedElementCounter& InOutCounter)
{
	check(InObject);
	InOutCounter.DecrementCounter(NAME_Class, InObject->GetClass());
}

