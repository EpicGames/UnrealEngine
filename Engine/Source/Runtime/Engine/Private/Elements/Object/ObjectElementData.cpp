// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Object/ObjectElementData.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/Stack.h"

UE_DEFINE_TYPED_ELEMENT_DATA_RTTI(FObjectElementData);

namespace ObjectElementDataUtil
{

UObject* GetObjectFromHandle(const FTypedElementHandle& InHandle, const bool bSilent)
{
	const FObjectElementData* ObjectElement = InHandle.GetData<FObjectElementData>(bSilent);
	return ObjectElement ? ObjectElement->Object : nullptr;
}

UObject* GetObjectFromHandleChecked(const FTypedElementHandle& InHandle)
{
	const FObjectElementData& ObjectElement = InHandle.GetDataChecked<FObjectElementData>();
	return ObjectElement.Object;
}

} // namespace ObjectElementDataUtil
