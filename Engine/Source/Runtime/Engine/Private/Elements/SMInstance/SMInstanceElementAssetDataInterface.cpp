// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementAssetDataInterface.h"

#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementAssetDataInterface)

FAssetData USMInstanceElementAssetDataInterface::GetAssetData(const FTypedElementHandle& InElementHandle)
{
	if (FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		if (UStaticMesh* StaticMesh = SMInstance.GetISMComponent()->GetStaticMesh())
		{
			return FAssetData(StaticMesh);
		}
	}

	return FAssetData();
}

