// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "ComponentRecreateRenderStateContext.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "UObject/Stack.h"

UE_DEFINE_TYPED_ELEMENT_DATA_RTTI(FSMInstanceElementData);

namespace SMInstanceElementDataUtil
{

static int32 GEnableSMInstanceElements = 1;
static FAutoConsoleVariableRef CVarEnableSMInstanceElements(
	TEXT("TypedElements.EnableSMInstanceElements"),
	GEnableSMInstanceElements,
	TEXT("Is support for static mesh instance elements enabled?"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
		SMInstanceElementDataUtil::OnSMInstanceElementsEnabledChanged().Broadcast();
	})
);

FSimpleMulticastDelegate& OnSMInstanceElementsEnabledChanged()
{
	static FSimpleMulticastDelegate OnInstanceElementsEnabledChanged;
	return OnInstanceElementsEnabledChanged;
}

bool SMInstanceElementsEnabled()
{
	return GEnableSMInstanceElements != 0;
}

ISMInstanceManager* GetSMInstanceManager(const FSMInstanceId& InstanceId)
{
	if (!InstanceId)
	{
		return nullptr;
	}

	if (AActor* OwnerActor = InstanceId.ISMComponent->GetOwner())
	{
		// If the owner actor is an instance manager provider, then just ask that for the instance manager
		if (ISMInstanceManagerProvider* InstanceManagerProvider = Cast<ISMInstanceManagerProvider>(OwnerActor))
		{
			return InstanceManagerProvider->GetSMInstanceManager(InstanceId);
		}

		// If the owner actor is an instance manager, then just use that
		if (ISMInstanceManager* InstanceManager = Cast<ISMInstanceManager>(OwnerActor))
		{
			return InstanceManager;
		}
	}

	// Otherwise, allow the ISM component to manage itself
	return InstanceId.ISMComponent;
}

FSMInstanceManager GetSMInstanceFromHandle(const FTypedElementHandle& InHandle, const bool bSilent)
{
	const FSMInstanceElementData* SMInstanceElement = InHandle.GetData<FSMInstanceElementData>(bSilent);
	const FSMInstanceId SMInstanceId = SMInstanceElement ? FSMInstanceElementIdMap::Get().GetSMInstanceIdFromSMInstanceElementId(SMInstanceElement->InstanceElementId) : FSMInstanceId();
	return FSMInstanceManager(SMInstanceId, GetSMInstanceManager(SMInstanceId));
}

FSMInstanceManager GetSMInstanceFromHandleChecked(const FTypedElementHandle& InHandle)
{
	const FSMInstanceElementData& SMInstanceElement = InHandle.GetDataChecked<FSMInstanceElementData>();
	const FSMInstanceId SMInstanceId = FSMInstanceElementIdMap::Get().GetSMInstanceIdFromSMInstanceElementId(SMInstanceElement.InstanceElementId);
	return FSMInstanceManager(SMInstanceId, GetSMInstanceManager(SMInstanceId));
}

} // namespace SMInstanceElementDataUtil
