// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementHierarchyInterface.h"

#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Elements/Framework/EngineElementsLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementHierarchyInterface)

FTypedElementHandle USMInstanceElementHierarchyInterface::GetParentElement(const FTypedElementHandle& InElementHandle, const bool bAllowCreate)
{
	if (FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
#if WITH_EDITOR
		if (UInstancedStaticMeshComponent* ISMComponent = SMInstance.GetISMComponent())
		{
			return UEngineElementsLibrary::AcquireEditorComponentElementHandle(ISMComponent, bAllowCreate);
		}
#endif	// WITH_EDITOR
	}
	return FTypedElementHandle();
}

