// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementId.h"

#include "Templates/Casts.h"
#include "UObject/Package.h"
#include "Components/InstancedStaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementId)

#if WITH_EDITORONLY_DATA
#include "Serialization/TextReferenceCollector.h"
#endif //WITH_EDITORONLY_DATA

const FName NAME_SMInstance = "SMInstance";

FSMInstanceElementIdMapEntry::FSMInstanceElementIdMapEntry(FSMInstanceElementIdMap* InOwner, UInstancedStaticMeshComponent* InComponent)
	: Owner(InOwner)
	, Component(InComponent)
{
	checkf(Owner, TEXT("FSMInstanceElementIdMapEntry must be constructed with a valid owner!"));
	checkf(Component, TEXT("FSMInstanceElementIdMapEntry must be constructed with a valid component!"));

#if WITH_EDITORONLY_DATA
	Transactor = NewObject<USMInstanceElementIdMapTransactor>(GetTransientPackage(), NAME_None, RF_Transient);
	Transactor->SetFlags(RF_Transactional);
	Transactor->SetOwnerEntry(this);
#endif	// WITH_EDITORONLY_DATA
}

FSMInstanceElementIdMapEntry::~FSMInstanceElementIdMapEntry()
{
#if WITH_EDITORONLY_DATA
	Transactor->SetOwnerEntry(nullptr);
#endif	// WITH_EDITORONLY_DATA
}


FSMInstanceElementIdMap& FSMInstanceElementIdMap::Get()
{
	static FSMInstanceElementIdMap Instance;
	return Instance;
}

FSMInstanceElementIdMap::~FSMInstanceElementIdMap()
{
	UnregisterCallbacks();
}

FSMInstanceId FSMInstanceElementIdMap::GetSMInstanceIdFromSMInstanceElementId(const FSMInstanceElementId& InSMInstanceElementId)
{
	if (InSMInstanceElementId)
	{
		FScopeLock Lock(&ISMComponentsCS);

		if (TSharedPtr<FSMInstanceElementIdMapEntry> ISMEntry = ISMComponents.FindRef(InSMInstanceElementId.ISMComponent))
		{
			if (const int32* InstanceIndexPtr = ISMEntry->InstanceIdToIndexMap.Find(InSMInstanceElementId.InstanceId))
			{
				return FSMInstanceId{ InSMInstanceElementId.ISMComponent, *InstanceIndexPtr };
			}
		}
	}

	return FSMInstanceId();
}

FSMInstanceElementId FSMInstanceElementIdMap::GetSMInstanceElementIdFromSMInstanceId(const FSMInstanceId& InSMInstanceId, const bool bAllowCreate)
{
	if (InSMInstanceId)
	{
		FScopeLock Lock(&ISMComponentsCS);

		TSharedPtr<FSMInstanceElementIdMapEntry> ISMEntry = ISMComponents.FindRef(InSMInstanceId.ISMComponent);
		if (ISMEntry)
		{
			if (const uint64* InstanceIdPtr = ISMEntry->InstanceIndexToIdMap.Find(InSMInstanceId.InstanceIndex))
			{
				return FSMInstanceElementId{ InSMInstanceId.ISMComponent, *InstanceIdPtr };
			}
		}

		if (bAllowCreate)
		{
			if (!ISMEntry)
			{
				ISMEntry = ISMComponents.Add(InSMInstanceId.ISMComponent, MakeShared<FSMInstanceElementIdMapEntry>(this, InSMInstanceId.ISMComponent));
				RegisterCallbacks();
			}

			checkf(ISMEntry->NextInstanceId < MAX_uint64, TEXT("Ran out of instance IDs for ISM component '%s'!"), *InSMInstanceId.ISMComponent->GetPathName());
			uint64 InstanceId = ISMEntry->NextInstanceId++;

			ISMEntry->InstanceIndexToIdMap.Add(InSMInstanceId.InstanceIndex, InstanceId);
			ISMEntry->InstanceIdToIndexMap.Add(InstanceId, InSMInstanceId.InstanceIndex);

			return FSMInstanceElementId{ InSMInstanceId.ISMComponent, InstanceId };
		}
	}

	return FSMInstanceElementId();
}

TArray<FSMInstanceElementId> FSMInstanceElementIdMap::GetSMInstanceElementIdsForComponent(UInstancedStaticMeshComponent* InComponent) const
{
	TArray<FSMInstanceElementId> SMInstanceElementIds;

	{
		FScopeLock Lock(&ISMComponentsCS);

		if (TSharedPtr<const FSMInstanceElementIdMapEntry> ISMEntry = ISMComponents.FindRef(InComponent))
		{
			SMInstanceElementIds.Reserve(ISMEntry->InstanceIdToIndexMap.Num());
			for (const TTuple<uint64, int32>& InstanceIdToIndexPair : ISMEntry->InstanceIdToIndexMap)
			{
				SMInstanceElementIds.Add(FSMInstanceElementId{ InComponent, InstanceIdToIndexPair.Key });
			}
		}
	}

	return SMInstanceElementIds;
}

void FSMInstanceElementIdMap::OnComponentReplaced(UInstancedStaticMeshComponent* InOldComponent, UInstancedStaticMeshComponent* InNewComponent)
{
	FScopeLock Lock(&ISMComponentsCS);

	if (TSharedPtr<const FSMInstanceElementIdMapEntry> OldISMEntry = ISMComponents.FindRef(InOldComponent))
	{
		TSharedPtr<FSMInstanceElementIdMapEntry> NewISMEntry = ISMComponents.FindRef(InNewComponent);
		if (NewISMEntry)
		{
			ClearInstanceData_NoLock(InNewComponent, *NewISMEntry);
		}
		else
		{
			NewISMEntry = ISMComponents.Add(InNewComponent, MakeShared<FSMInstanceElementIdMapEntry>(this, InNewComponent));
		}

		NewISMEntry->InstanceIndexToIdMap.Reserve(OldISMEntry->InstanceIndexToIdMap.Num());
		NewISMEntry->InstanceIdToIndexMap.Reserve(OldISMEntry->InstanceIdToIndexMap.Num());
		NewISMEntry->NextInstanceId = OldISMEntry->NextInstanceId;

		for (const TTuple<uint64, int32>& OldInstanceIdToIndexPair : OldISMEntry->InstanceIdToIndexMap)
		{
			if (InNewComponent->IsValidInstance(OldInstanceIdToIndexPair.Value))
			{
				NewISMEntry->InstanceIndexToIdMap.Add(OldInstanceIdToIndexPair.Value, OldInstanceIdToIndexPair.Key);
				NewISMEntry->InstanceIdToIndexMap.Add(OldInstanceIdToIndexPair.Key, OldInstanceIdToIndexPair.Value);
			}
		}
	}
}

void FSMInstanceElementIdMap::SerializeIdMappings(FSMInstanceElementIdMapEntry* InEntry, FArchive& Ar)
{
	FScopeLock Lock(&ISMComponentsCS);

	TMap<uint64, int32> PreviousInstanceIdToIndexMap;
	if (Ar.IsLoading() && (OnInstanceRemappedDelegate.IsBound() || OnInstanceRemovedDelegate.IsBound()))
	{
		// Take a copy of the current instance IDs, so that we can find out which elements have been unmapped or remapped post-load
		PreviousInstanceIdToIndexMap = InEntry->InstanceIdToIndexMap;
	}

	// The two maps are mirrors of each other, so we only need to serialize one as we can restore the other again on load
	Ar << InEntry->InstanceIdToIndexMap;
	if (Ar.IsLoading())
	{
		InEntry->InstanceIndexToIdMap.Reset();
		InEntry->InstanceIndexToIdMap.Reserve(InEntry->InstanceIdToIndexMap.Num());
		for (const TTuple<uint64, int32>& InstanceIdToIndexPair : InEntry->InstanceIdToIndexMap)
		{
			InEntry->InstanceIndexToIdMap.Add(InstanceIdToIndexPair.Value, InstanceIdToIndexPair.Key);
		}
	}

	if (Ar.IsLoading() && (OnInstanceRemappedDelegate.IsBound() || OnInstanceRemovedDelegate.IsBound()))
	{
		// Emit events for elements that were unmapped or remapped from this load
		for (const TTuple<uint64, int32>& PreviousInstanceIdToIndexPair : PreviousInstanceIdToIndexMap)
		{
			if (const int32* InstanceIndexPtr = InEntry->InstanceIdToIndexMap.Find(PreviousInstanceIdToIndexPair.Key))
			{
				if (PreviousInstanceIdToIndexPair.Value != *InstanceIndexPtr)
				{
					OnInstanceRemappedDelegate.Broadcast(FSMInstanceElementId{ InEntry->Component, PreviousInstanceIdToIndexPair.Key }, PreviousInstanceIdToIndexPair.Value, *InstanceIndexPtr);
				}
			}
			else
			{
				OnInstanceRemovedDelegate.Broadcast(FSMInstanceElementId{ InEntry->Component, PreviousInstanceIdToIndexPair.Key }, PreviousInstanceIdToIndexPair.Value);
			}
		}
		if (OnInstanceRemappedDelegate.IsBound())
		{
			for (const TTuple<uint64, int32>& InstanceIdToIndexPair : InEntry->InstanceIdToIndexMap)
			{
				if (!PreviousInstanceIdToIndexMap.Contains(InstanceIdToIndexPair.Key))
				{
					OnInstanceRemappedDelegate.Broadcast(FSMInstanceElementId{ InEntry->Component, InstanceIdToIndexPair.Key }, INDEX_NONE, InstanceIdToIndexPair.Value);
				}
			}
		}
	}
}

void FSMInstanceElementIdMap::OnInstanceIndexUpdated(UInstancedStaticMeshComponent* InComponent, TArrayView<const FInstancedStaticMeshDelegates::FInstanceIndexUpdateData> InIndexUpdates)
{
	FScopeLock Lock(&ISMComponentsCS);

	TSharedPtr<FSMInstanceElementIdMapEntry> ISMEntry = ISMComponents.FindRef(InComponent);
	if (!ISMEntry)
	{
		// We don't process 'Added' updates below, so we can bail immediately if the component in question isn't registered
		return;
	}

	for (const FInstancedStaticMeshDelegates::FInstanceIndexUpdateData& IndexUpdateData : InIndexUpdates)
	{
		switch (IndexUpdateData.Type)
		{
		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Added:
			break; // We don't need to process 'Added' updates as they can't affect existing IDs

		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Removed:
			{
				if (const uint64* InstanceIdPtr = ISMEntry->InstanceIndexToIdMap.Find(IndexUpdateData.Index))
				{
					const uint64 InstanceId = *InstanceIdPtr;

					OnInstancePreRemovalDelegate.Broadcast(FSMInstanceElementId{ InComponent, InstanceId }, IndexUpdateData.Index);

					ISMEntry->InstanceIndexToIdMap.Remove(IndexUpdateData.Index);
					ISMEntry->InstanceIdToIndexMap.Remove(InstanceId);

					OnInstanceRemovedDelegate.Broadcast(FSMInstanceElementId{ InComponent, InstanceId }, IndexUpdateData.Index);
				}
				// Note: We can't remove things from the ISMComponents map until we get a 'Destroyed' notification for the owner ISM component, as the 
				// per-component transactors may be referenced by the undo buffer, so the transactor needs to persist in order for redo to work correctly
			}
			break;

		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Relocated:
			{
				if (const uint64* InstanceIdPtr = ISMEntry->InstanceIndexToIdMap.Find(IndexUpdateData.OldIndex))
				{
					const uint64 InstanceId = *InstanceIdPtr;

					ISMEntry->InstanceIndexToIdMap.Remove(IndexUpdateData.OldIndex);
					ISMEntry->InstanceIndexToIdMap.Add(IndexUpdateData.Index, InstanceId);
					ISMEntry->InstanceIdToIndexMap[InstanceId] = IndexUpdateData.Index;

					OnInstanceRemappedDelegate.Broadcast(FSMInstanceElementId{ InComponent, InstanceId }, IndexUpdateData.OldIndex, IndexUpdateData.Index);
				}
			}
			break;

		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Cleared:
			{
				ClearInstanceData_NoLock(InComponent, *ISMEntry);
			}
			break;

		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Destroyed:
			{
				ISMComponents.Remove(InComponent);

				if (OnInstanceRemovedDelegate.IsBound())
				{
					for (const TTuple<uint64, int32>& InstanceIdToIndexPair : ISMEntry->InstanceIdToIndexMap)
					{
						OnInstanceRemovedDelegate.Broadcast(FSMInstanceElementId{ InComponent, InstanceIdToIndexPair.Key }, InstanceIdToIndexPair.Value);
					}
				}
			}
			break;

		default:
			break;
		}
	}

	if (ISMComponents.Num() == 0)
	{
		UnregisterCallbacks();
	}
}

void FSMInstanceElementIdMap::ClearInstanceData_NoLock(UInstancedStaticMeshComponent* InComponent, FSMInstanceElementIdMapEntry& InEntry)
{
	if (OnInstancePreRemovalDelegate.IsBound())
	{
		for (const TTuple<uint64, int32>& InstanceIdToIndexPair : InEntry.InstanceIdToIndexMap)
		{
			OnInstancePreRemovalDelegate.Broadcast(FSMInstanceElementId{ InComponent, InstanceIdToIndexPair.Key }, InstanceIdToIndexPair.Value);
		}
	}

	// Take the current instance IDs, so that we can find notify the elements have been unmapped
	TMap<uint64, int32> PreviousInstanceIdToIndexMap = MoveTemp(InEntry.InstanceIdToIndexMap);

	// Note: We can't remove things from the ISMComponents map until we get a 'Destroyed' notification for the owner ISM component, as the 
	// per-component transactors may be referenced by the undo buffer, so the transactor needs to persist in order for redo to work correctly
	InEntry.InstanceIndexToIdMap.Reset();
	InEntry.InstanceIdToIndexMap.Reset();

	if (OnInstanceRemovedDelegate.IsBound())
	{
		for (const TTuple<uint64, int32>& PreviousInstanceIdToIndexPair : PreviousInstanceIdToIndexMap)
		{
			OnInstanceRemovedDelegate.Broadcast(FSMInstanceElementId{ InComponent, PreviousInstanceIdToIndexPair.Key }, PreviousInstanceIdToIndexPair.Value);
		}
	}
}

#if WITH_EDITOR
void FSMInstanceElementIdMap::OnObjectModified(UObject* InObject)
{
	if (UInstancedStaticMeshComponent* Component = Cast<UInstancedStaticMeshComponent>(InObject))
	{
		FScopeLock Lock(&ISMComponentsCS);

		if (TSharedPtr<FSMInstanceElementIdMapEntry> ISMEntry = ISMComponents.FindRef(Component))
		{
			// The component has been modified, so modify our transactor too to keep the maps in sync over an undo/redo
			ISMEntry->Transactor->Modify();
		}
	}
}
#endif	// WITH_EDITOR

void FSMInstanceElementIdMap::RegisterCallbacks()
{
	if (!OnInstanceIndexUpdatedHandle.IsValid())
	{
		OnInstanceIndexUpdatedHandle = FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.AddRaw(this, &FSMInstanceElementIdMap::OnInstanceIndexUpdated);
	}

#if WITH_EDITOR
	if (!OnObjectModifiedHandle.IsValid())
	{
		OnObjectModifiedHandle = FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FSMInstanceElementIdMap::OnObjectModified);
	}
#endif	// WITH_EDITOR
}

void FSMInstanceElementIdMap::UnregisterCallbacks()
{
	if (OnInstanceIndexUpdatedHandle.IsValid())
	{
		FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.Remove(OnInstanceIndexUpdatedHandle);
		OnInstanceIndexUpdatedHandle.Reset();
	}

#if WITH_EDITOR
	if (OnObjectModifiedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectModified.Remove(OnObjectModifiedHandle);
		OnObjectModifiedHandle.Reset();
	}
#endif	// WITH_EDITOR
}

void FSMInstanceElementIdMap::AddReferencedObjects(FReferenceCollector& Collector)
{
#if WITH_EDITORONLY_DATA
	FScopeLock Lock(&ISMComponentsCS);

	Collector.AllowEliminatingReferences(false);
	for (const auto& ISMEntryPair : ISMComponents)
	{
		Collector.AddReferencedObject(ISMEntryPair.Value->Transactor);
	}
	Collector.AllowEliminatingReferences(true);
#endif	// WITH_EDITORONLY_DATA
}

USMInstanceElementIdMapTransactor::USMInstanceElementIdMapTransactor()
{
#if WITH_EDITORONLY_DATA
	// The map transactor should not get text reference collected, as it is native and transient only, and should not find its way into asset packages
	{ static const FAutoRegisterTextReferenceCollectorCallback AutomaticRegistrationOfTextReferenceCollector(USMInstanceElementIdMapTransactor::StaticClass(), [](UObject* Object, FArchive& Ar) {}); }
#endif //WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
void USMInstanceElementIdMapTransactor::Serialize(FArchive& Ar)
{
	checkf(!Ar.IsPersistent() || this->HasAnyFlags(RF_ClassDefaultObject),
		TEXT("USMInstanceElementIdMapTransactor can only be serialized by transient archives!"));

	if (OwnerEntry)
	{
		OwnerEntry->Owner->SerializeIdMappings(OwnerEntry, Ar);
	}
}
#endif	// WITH_EDITORONLY_DATA

