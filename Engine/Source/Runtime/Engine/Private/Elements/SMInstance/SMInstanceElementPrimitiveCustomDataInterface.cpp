// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementPrimitiveCustomDataInterface.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Elements/SMInstance/SMInstanceElementData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementPrimitiveCustomDataInterface)

void USMInstanceElementPrimitiveCustomDataInterface::SetCustomData(const FTypedElementHandle& InElementHandle,
	TArrayView<const float> CustomDataFloats, bool bMarkRenderStateDirty)
{
	FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	if (SMInstance)
	{
		SMInstance.GetISMComponent()->SetCustomData(SMInstance.GetISMInstanceIndex(), CustomDataFloats, bMarkRenderStateDirty);
	}
}

void USMInstanceElementPrimitiveCustomDataInterface::SetCustomDataValue(const FTypedElementHandle& InElementHandle,
	int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty)
{
	FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	if (SMInstance)
	{
		SMInstance.GetISMComponent()->SetCustomDataValue(SMInstance.GetISMInstanceIndex(), CustomDataIndex, CustomDataValue, bMarkRenderStateDirty);
	}
}
