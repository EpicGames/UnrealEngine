// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementSelectionInterface.h"
#include "Elements/SMInstance/SMInstanceElementData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementSelectionInterface)

bool USMInstanceElementSelectionInterface::SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	if (SMInstance && ITypedElementSelectionInterface::SelectElement(InElementHandle, InSelectionSet, InSelectionOptions))
	{
		SMInstance.NotifySMInstanceSelectionChanged(/*bIsSelected*/true);
		return true;
	}
	return false;
}

bool USMInstanceElementSelectionInterface::DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	if (SMInstance && ITypedElementSelectionInterface::DeselectElement(InElementHandle, InSelectionSet, InSelectionOptions))
	{
		SMInstance.NotifySMInstanceSelectionChanged(/*bIsSelected*/false);
		return true;
	}
	return false;
}

