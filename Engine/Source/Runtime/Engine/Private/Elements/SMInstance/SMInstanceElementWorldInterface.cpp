// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementWorldInterface.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/SMInstance/SMInstanceElementData.h"

#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"
#include "ShowFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementWorldInterface)

bool USMInstanceElementWorldInterface::CanEditElement(const FTypedElementHandle& InElementHandle)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.CanEditSMInstance();
}

bool USMInstanceElementWorldInterface::IsTemplateElement(const FTypedElementHandle& InElementHandle)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.GetISMComponent()->IsTemplate();
}

ULevel* USMInstanceElementWorldInterface::GetOwnerLevel(const FTypedElementHandle& InElementHandle)
{
	if (const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		if (const AActor* ComponentOwner = SMInstance.GetISMComponent()->GetOwner())
		{
			return ComponentOwner->GetLevel();
		}
	}

	return nullptr;
}

UWorld* USMInstanceElementWorldInterface::GetOwnerWorld(const FTypedElementHandle& InElementHandle)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance ? SMInstance.GetISMComponent()->GetWorld() : nullptr;
}

bool USMInstanceElementWorldInterface::GetBounds(const FTypedElementHandle& InElementHandle, FBoxSphereBounds& OutBounds)
{
	if (const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		if (const UStaticMesh* StaticMesh = SMInstance.GetISMComponent()->GetStaticMesh())
		{
			OutBounds = StaticMesh->GetBounds();
		}
		else
		{
			OutBounds = FBoxSphereBounds();
		}

		FTransform InstanceTransform;
		SMInstance.GetSMInstanceTransform(InstanceTransform, /*bWorldSpace*/true);

		OutBounds = OutBounds.TransformBy(InstanceTransform);
		return true;
	}

	return false;
}

bool USMInstanceElementWorldInterface::CanMoveElement(const FTypedElementHandle& InElementHandle, const ETypedElementWorldType InWorldType)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.CanMoveSMInstance(InWorldType);
}

bool USMInstanceElementWorldInterface::GetWorldTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.GetSMInstanceTransform(OutTransform, /*bWorldSpace*/true);
}

bool USMInstanceElementWorldInterface::SetWorldTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.SetSMInstanceTransform(InTransform, /*bWorldSpace*/true, /*bMarkRenderStateDirty*/true);
}

bool USMInstanceElementWorldInterface::GetRelativeTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.GetSMInstanceTransform(OutTransform, /*bWorldSpace*/false);
}

bool USMInstanceElementWorldInterface::SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.SetSMInstanceTransform(InTransform, /*bWorldSpace*/false, /*bMarkRenderStateDirty*/true);
}

void USMInstanceElementWorldInterface::NotifyMovementStarted(const FTypedElementHandle& InElementHandle)
{
	if (FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		SMInstance.NotifySMInstanceMovementStarted();
	}
}

void USMInstanceElementWorldInterface::NotifyMovementOngoing(const FTypedElementHandle& InElementHandle)
{
	if (FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		SMInstance.NotifySMInstanceMovementOngoing();
	}
}

void USMInstanceElementWorldInterface::NotifyMovementEnded(const FTypedElementHandle& InElementHandle)
{
	if (FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		SMInstance.NotifySMInstanceMovementEnded();
	}
}

TArray<FTypedElementHandle> USMInstanceElementWorldInterface::GetSelectionElementsFromSelectionFunction(const FTypedElementHandle& InElementHandle, const FWorldSelectionElementArgs& SelectionArgs, const TFunction<bool(const FTypedElementHandle&, const FWorldSelectionElementArgs&)>& SelectionFunction)
{
	if (SelectionArgs.bBSPSelectionOnly && SelectionArgs.ShowFlags && !SelectionArgs.ShowFlags->StaticMeshes)
	{
		return {};
	}

	if (SelectionFunction(InElementHandle, SelectionArgs))
	{
		if (SelectionArgs.SelectionSet)
		{
			return {SelectionArgs.SelectionSet->GetSelectionElement(InElementHandle, SelectionArgs.SelectionMethod)};
		}
		else
		{
			return {InElementHandle};
		}
	}

	return {};
}

