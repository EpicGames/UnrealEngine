// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ActorInstanceHandle.h"
#include "Engine/ActorInstanceManagerInterface.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "UObject/FortniteValkyrieBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorInstanceHandle)

namespace UE::FActorInstanceHandle::Private
{
bool bValidateAccessFromGameThread = false;
static FAutoConsoleVariableRef CVarValidateAccessFromGameThread(
	TEXT("IA.ValidateAccessFromGameThread"),
	bValidateAccessFromGameThread,
	TEXT("If set errors will get reported when trying to resolve or access the handle from non game threads."),
	ECVF_Default);
} // UE::FActorInstanceHandle::Private

#define VALIDATE_ACCESS() \
	if (UE::FActorInstanceHandle::Private::bValidateAccessFromGameThread) \
	{ \
		ensureAlwaysMsgf(IsInGameThread() || IsInParallelGameThread(), TEXT("%hs can only be called from the game thread"), __FUNCTION__); \
	}


//-----------------------------------------------------------------------------
// FActorInstanceHandleInternalHelper
//-----------------------------------------------------------------------------
struct FActorInstanceHandleInternalHelper
{
	static void SetUpAsInterface(FActorInstanceHandle& InstanceHandle, IActorInstanceManagerInterface& InManagerInterface, const UPrimitiveComponent* RelevantComponent, const int32 CollisionInstanceIndex)
	{
		checkf(InstanceHandle.ResolutionStatus == FActorInstanceHandle::EResolutionStatus::Invalid, TEXT("Expected to be called on a newly constructed handle."));
		InstanceHandle.InstanceIndex = InManagerInterface.ConvertCollisionIndexToInstanceIndex(CollisionInstanceIndex, RelevantComponent);
		InstanceHandle.ReferenceObject = InManagerInterface.FindActor(InstanceHandle);
		InstanceHandle.ResolutionStatus = FActorInstanceHandle::EResolutionStatus::Resolved;
	}

	static void SetUpWithActor(FActorInstanceHandle& InstanceHandle, AActor* InActor, const UPrimitiveComponent* RelevantComponent, const int32 CollisionInstanceIndex)
	{
		checkf(InstanceHandle.ResolutionStatus == FActorInstanceHandle::EResolutionStatus::Invalid, TEXT("Expected to be called on a newly constructed handle."));
		InstanceHandle.ManagerInterface = FActorInstanceManagerInterface(InActor);
		if (IActorInstanceManagerInterface* ManagerInterfacePtr = InstanceHandle.ManagerInterface.Get())
		{
			SetUpAsInterface(InstanceHandle, *ManagerInterfacePtr, RelevantComponent, CollisionInstanceIndex);
		}
		else
		{
			InstanceHandle.ReferenceObject = InActor;
		}
		InstanceHandle.ResolutionStatus = FActorInstanceHandle::EResolutionStatus::Resolved;
	}
};

//-----------------------------------------------------------------------------
// FActorInstanceHandle
//-----------------------------------------------------------------------------
FActorInstanceHandle::FActorInstanceHandle(AActor* InActor)
	: ReferenceObject(InActor)
	, ResolutionStatus(EResolutionStatus::Resolved)
{
	VALIDATE_ACCESS();
}

FActorInstanceHandle::FActorInstanceHandle(const UPrimitiveComponent* RelevantComponent, const int32 CollisionInstanceIndex)
{
	VALIDATE_ACCESS();

	if (UNLIKELY(!ensureMsgf(RelevantComponent, TEXT("Calling FActorInstanceHandle(UPrimitiveComponent, int32) constructor is pointless with RelevantComponent == nullptr"))))
	{
		return;
	}

	if (AActor* OwnerActor = RelevantComponent->GetOwner())
	{
		FActorInstanceHandleInternalHelper::SetUpWithActor(*this, OwnerActor, RelevantComponent, CollisionInstanceIndex);
	}
}

FActorInstanceHandle::FActorInstanceHandle(AActor* InActor, const UPrimitiveComponent* RelevantComponent, const int32 CollisionInstanceIndex)
{
	VALIDATE_ACCESS();

	if (LIKELY(InActor))
	{
		FActorInstanceHandleInternalHelper::SetUpWithActor(*this, InActor, RelevantComponent, CollisionInstanceIndex);
	}
	else if (RelevantComponent)
	{
		*this = FActorInstanceHandle(RelevantComponent, CollisionInstanceIndex);
	}
}

FActorInstanceHandle::FActorInstanceHandle(const FActorInstanceManagerInterface InManagerInterface, const int32 CollisionInstanceIndex)
	: ManagerInterface(InManagerInterface)
{
	VALIDATE_ACCESS();

	if (IActorInstanceManagerInterface* ManagerInterfacePtr = ManagerInterface.Get())
	{
		FActorInstanceHandleInternalHelper::SetUpAsInterface(*this, *ManagerInterfacePtr, /*RelevantComponent=*/nullptr, CollisionInstanceIndex);
	}
}

FActorInstanceHandle::FActorInstanceHandle(const FActorInstanceHandle& Other)
{
	ReferenceObject = Other.ReferenceObject;
	ManagerInterface = Other.ManagerInterface;
	InstanceIndex = Other.InstanceIndex;
	ResolutionStatus = Other.ResolutionStatus;
}

FActorInstanceHandle FActorInstanceHandle::MakeDehydratedActorHandle(UObject& Manager, const int32 InInstanceIndex)
{
	FActorInstanceHandle ReturnHandle;
	ReturnHandle.ManagerInterface = FActorInstanceManagerInterface(&Manager);
	ReturnHandle.InstanceIndex = InInstanceIndex;
	ReturnHandle.ResolutionStatus = EResolutionStatus::Resolved;

	return ReturnHandle;
}

FActorInstanceHandle FActorInstanceHandle::MakeActorHandleToResolve(const TWeakObjectPtr<UPrimitiveComponent>& WeakComponent, const int32 CollisionInstanceIndex)
{
	ensureMsgf(!WeakComponent.IsExplicitlyNull(), TEXT("Provided weak pointer must be initialized."));
	FActorInstanceHandle ReturnHandle;
	ReturnHandle.ReferenceObject = WeakComponent;
	ReturnHandle.InstanceIndex = CollisionInstanceIndex;
	ReturnHandle.ResolutionStatus = EResolutionStatus::NeedsResolving;

	return ReturnHandle;
}

void FActorInstanceHandle::ResolveHandle() const
{
	if (ResolutionStatus == EResolutionStatus::NeedsResolving)
	{
		VALIDATE_ACCESS();

		FActorInstanceHandle* MutableThis = const_cast<FActorInstanceHandle*>(this);

		if (const UPrimitiveComponent* Component = CastChecked<UPrimitiveComponent>(ReferenceObject.Get(), ECastCheckedType::NullAllowed))
		{
			if (AActor* OwnerActor = Component->GetOwner())
			{
				// Reset resolution status to 'Invalid' since 'SetUpWithActor' requires handle to be in that status
				MutableThis->ResolutionStatus = EResolutionStatus::Invalid;

				// Resolving by using valid Actor, Component and collision instance index.
				FActorInstanceHandleInternalHelper::SetUpWithActor(*MutableThis, OwnerActor, Component, InstanceIndex);
				ensure(ResolutionStatus == EResolutionStatus::Resolved);
			}
		}

		// Reset the handle if we were unable to resolve it (e.g. component no longer valid or missing owner actor)
		if (ResolutionStatus == EResolutionStatus::NeedsResolving)
		{
			// OwnerActor is not valid so we need to reset the handle
			new (MutableThis)(FActorInstanceHandle);
		}
	}
}

bool FActorInstanceHandle::IsValid() const
{
	if (ResolutionStatus == EResolutionStatus::NeedsResolving)
	{
		if (IsInGameThread() || IsInParallelGameThread())
		{
			ResolveHandle();
		}
		else
		{
			// A handle properly setup from another thread that needs resolving is considered valid.
			return !ReferenceObject.IsExplicitlyNull();
		}
	}

	return (ManagerInterface.IsValid() && InstanceIndex != INDEX_NONE) || IsActorValid();
}

bool FActorInstanceHandle::IsUnresolvedDehydratedActor() const
{
	if (ResolutionStatus == EResolutionStatus::NeedsResolving)
	{
		if (const UPrimitiveComponent* Component = CastChecked<UPrimitiveComponent>(ReferenceObject.Get(), ECastCheckedType::NullAllowed))
		{
			if (AActor* OwnerActor = Component->GetOwner())
			{
				if (OwnerActor->GetClass()->ImplementsInterface(UActorInstanceManagerInterface::StaticClass()))
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FActorInstanceHandle::DoesRepresentClass(const UClass* OtherClass) const
{
	if (const UClass* RepresentedClass = GetRepresentedClass())
	{
		return RepresentedClass->IsChildOf(OtherClass);
	}
	return false;
}

UClass* FActorInstanceHandle::GetRepresentedClass() const
{
	// Calling IsActorValid will resolve the handle if necessary
	if (const AActor* CachedActor = GetCachedActor())
	{
		return CachedActor->GetClass();
	}

	IActorInstanceManagerInterface* ManagerInterfacePtr = ManagerInterface.Get();
	return ManagerInterfacePtr && (InstanceIndex != INDEX_NONE)
		? ManagerInterfacePtr->GetRepresentedClass(InstanceIndex)
		: nullptr;
}

ULevel* FActorInstanceHandle::GetLevel() const
{
	if (const AActor* Actor = GetCachedActor())
	{
		return Actor->GetLevel();
	}

	IActorInstanceManagerInterface* ManagerInterfacePtr = ManagerInterface.Get();
	return ManagerInterfacePtr && (InstanceIndex != INDEX_NONE)
		? ManagerInterfacePtr->GetLevelForInstance(InstanceIndex)
		: nullptr;
}

FVector FActorInstanceHandle::GetLocation() const
{
	if (const AActor* Actor = GetCachedActor())
	{
		return Actor->GetActorLocation();
	}

	IActorInstanceManagerInterface* ManagerInterfacePtr = ManagerInterface.Get();
	return ManagerInterfacePtr && (InstanceIndex != INDEX_NONE)
		? ManagerInterfacePtr->GetTransform(*this).GetLocation()
		: FVector();
}

FRotator FActorInstanceHandle::GetRotation() const
{
	if (const AActor* Actor = GetCachedActor())
	{
		return Actor->GetActorRotation();
	}

	IActorInstanceManagerInterface* ManagerInterfacePtr = ManagerInterface.Get();
	return ManagerInterfacePtr && (InstanceIndex != INDEX_NONE)
		? ManagerInterfacePtr->GetTransform(*this).GetRotation().Rotator()
		: FRotator();
}

FTransform FActorInstanceHandle::GetTransform() const
{
	if (const AActor* Actor = GetCachedActor())
	{
		return Actor->GetActorTransform();
	}

	IActorInstanceManagerInterface* ManagerInterfacePtr = ManagerInterface.Get();
	return ManagerInterfacePtr && (InstanceIndex != INDEX_NONE)
		? ManagerInterfacePtr->GetTransform(*this)
		: FTransform();
}

FName FActorInstanceHandle::GetFName() const
{
	if (const AActor* Actor = GetCachedActor())
	{
		return Actor->GetFName();
	}

	return NAME_None;
}

FString FActorInstanceHandle::GetName() const
{
	if (const AActor* Actor = GetCachedActor())
	{
		return Actor->GetName();
	}

	if (ManagerInterface.IsValid())
	{
		return FString::Printf(TEXT("%s:%d"), *GetNameSafe(ManagerInterface.GetObject()), InstanceIndex);
	}

	return TEXT("Invalid");
}

AActor* FActorInstanceHandle::GetManagingActor() const
{
	if (AActor* Actor = GetCachedActor())
	{
		return Actor;
	}

	return Cast<AActor>(ManagerInterface.GetObject());
}

USceneComponent* FActorInstanceHandle::GetRootComponent() const
{
	if (const AActor* Actor = GetCachedActor())
	{
		return Actor->GetRootComponent();
	}

	AActor* AsActor = Cast<AActor>(ManagerInterface.GetObject());
	return AsActor ? AsActor->GetRootComponent() : nullptr;
}

AActor* FActorInstanceHandle::FetchActor() const
{
	AActor* FetchedActor = GetCachedActor();
	if (FetchedActor != nullptr)
	{
		return FetchedActor;
	}

	if (ManagerInterface.IsValid())
	{
		FetchedActor = ManagerInterface->FindOrCreateActor(*this);
		SetCachedActor(FetchedActor);
	}

	return FetchedActor;
}

UObject* FActorInstanceHandle::GetActorAsUObject()
{
	return GetCachedActor();
}

const UObject* FActorInstanceHandle::GetActorAsUObject() const
{
	return GetCachedActor();
}

bool FActorInstanceHandle::IsActorValid() const
{
	return GetCachedActor() != nullptr;
}

AActor* FActorInstanceHandle::GetCachedActor() const
{
	// Make sure handle is resolved before getting the actor.
	ResolveHandle();

	return Cast<AActor>(ReferenceObject.Get());
}

void FActorInstanceHandle::SetCachedActor(AActor* InActor) const
{
	check(IsActorValid() == false);
	ReferenceObject = InActor;
}

FActorInstanceHandle& FActorInstanceHandle::operator=(AActor* OtherActor)
{
	new (this)FActorInstanceHandle(OtherActor);
	return *this;
}

bool FActorInstanceHandle::operator==(const FActorInstanceHandle& Other) const
{
	if (IsInGameThread() || IsInParallelGameThread())
	{
		// Both handles need to be resolved to perform a valid comparison but only on game thread.
		if (ResolutionStatus == EResolutionStatus::NeedsResolving)
		{
			ResolveHandle();
		}

		if (Other.ResolutionStatus == EResolutionStatus::NeedsResolving)
		{
			Other.ResolveHandle();
		}
	}
	else if (ResolutionStatus == EResolutionStatus::NeedsResolving
		&& Other.ResolutionStatus == EResolutionStatus::NeedsResolving)
	{
		// Handles that needs resolving can be compared using the reference object and index.
		return (ReferenceObject.HasSameIndexAndSerialNumber(Other.ReferenceObject)) && (InstanceIndex == Other.InstanceIndex);
	}
	else
	{
		ensureMsgf(false, TEXT("Comparing resolved and non-resolved handles is not supported."));
		return false;
	}

	// If both handles point to the same Manager/InstanceIndex pair, we can assume they'll point
	// to the same ReferenceObject.
	if ((InstanceIndex != INDEX_NONE && Other.InstanceIndex != INDEX_NONE) && (ManagerInterface == Other.ManagerInterface && InstanceIndex == Other.InstanceIndex))
	{
		return true;
	}

	// If both Actors are set, we can compare the pointers.
	// But if one Actor is missing, there's a possibility that the Actor was spawned after the handle was initially setup.
	// In that case we defer to operator==(AActor*) which will try to update the cached actor by calling FindActor on the manager interface.
	// If one of the actor is set and FindActor returns null, we know both handles can't point to the same Actor.
	const AActor* MyActor = GetCachedActor();
	const AActor* OtherActor = Other.GetCachedActor();
	if (MyActor == ManagerInterface.GetObject() && OtherActor == Other.ManagerInterface.GetObject())
	{
		// Both actors point to their own managers, so we can compare their Manager/InstanceIndex instead,
		// since two handles may live inside the same manager. In this case, we'll just return false, since
		// we already verified above that their Manager/InstanceIndex are different.
		return false;
	}
	
	if (MyActor)
	{
		if (OtherActor)
		{
			return Other == MyActor;
		}
		else
		{
			return Other.operator==(MyActor);
		}
	}
	else if (OtherActor)
	{
		return operator==(OtherActor);
	}

	return false;
}

bool FActorInstanceHandle::operator!=(const FActorInstanceHandle& Other) const
{
	return !(*this == Other);
}

bool FActorInstanceHandle::operator==(const AActor* OtherActor) const
{
	VALIDATE_ACCESS();

	// if we have an actor, compare the two actors
	if (const AActor* AsActor = GetCachedActor())
	{
		return AsActor == OtherActor;
	}

	// if OtherActor is null then we're only equal if this doesn't refer to a valid instance
	if (OtherActor == nullptr)
	{
		return !ManagerInterface.IsValid() && InstanceIndex == INDEX_NONE;
	}

	if (IActorInstanceManagerInterface* ManagerInterfacePtr = ManagerInterface.Get())
	{
		// Since we need to call FindActor, we might as well save the result.
		// Setting ReferencedObject without calling SetCachedActor to avoid having to const_cast this.
		ReferenceObject = ManagerInterfacePtr->FindActor(*this);
	}

	return GetCachedActor() == OtherActor;
}

bool FActorInstanceHandle::operator!=(const AActor* OtherActor) const
{
	return !(*this == OtherActor);
}

uint32 GetTypeHash(const FActorInstanceHandle& Handle)
{
	uint32 Hash = 0;
	if (const AActor* Actor = Handle.GetCachedActor())
	{
		Hash = FCrc::StrCrc32(*(Actor->GetPathName()), Hash);
	}
	if (UObject* ManagerInterfaceObject = Handle.ManagerInterface.GetObject())
	{
		Hash = HashCombine(Hash, GetTypeHash(ManagerInterfaceObject));
	}
	Hash = HashCombine(Hash, Handle.InstanceIndex);

	return Hash;
}

FArchive& operator<<(FArchive& Ar, FActorInstanceHandle& Handle)
{
	Ar.UsingCustomVersion(FFortniteValkyrieBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteValkyrieBranchObjectVersion::GUID) < FFortniteValkyrieBranchObjectVersion::ActorInstanceHandleSwitchedToInterfaces)
	{
		Ar << Handle.ReferenceObject;
		TWeakObjectPtr<AActor> Manager;
		Ar << Manager;
		ensureMsgf(Ar.IsLoading(), TEXT("We expect this piece of code to be running only while loading data."));
		Handle.ManagerInterface = FActorInstanceManagerInterface(Manager.Get());
		Ar << Handle.InstanceIndex;
		return Ar;
	}

	if (Ar.IsSaving())
	{
		Handle.ResolveHandle();
	}
	Ar << Handle.ReferenceObject;
	Ar << Handle.InstanceIndex;

	// Serializing as a TWeakObjectPtr instead of TSoftObjectPtr to avoid UE-205038.
	TWeakObjectPtr<UObject> WeakManagerObject(Handle.ManagerInterface.GetObject());
	Ar << WeakManagerObject;

	if (Ar.IsLoading())
	{
		Handle.ManagerInterface = FActorInstanceManagerInterface(WeakManagerObject.Get());
		IActorInstanceManagerInterface* AsManager = Handle.ManagerInterface.Get();

		if (AsManager && Handle.InstanceIndex != INDEX_NONE
			&& Handle.ManagerInterface.GetObject() == Handle.ReferenceObject)
		{
			Handle.ResolutionStatus = FActorInstanceHandle::EResolutionStatus::Invalid;
			Handle.ReferenceObject = nullptr;
			Handle.ReferenceObject = AsManager->FindActor(Handle);
		}
		Handle.ResolutionStatus = FActorInstanceHandle::EResolutionStatus::Resolved;
	}

	return Ar;
}
