// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/AutoDestroySubsystem.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutoDestroySubsystem)

DECLARE_LOG_CATEGORY_EXTERN(LogAutoDestroySubsystem, Log, All);
DEFINE_LOG_CATEGORY(LogAutoDestroySubsystem);

void UAutoDestroySubsystem::Deinitialize()
{
	// Ensure that the resources from the world are cleaned up and callbacks unregistered
	for (AActor* Actor : ActorsToPoll)
	{
		if (Actor)
		{
			Actor->OnEndPlay.RemoveAll(this);
		}
	}
	ActorsToPoll.Empty();

	Super::Deinitialize();
}

bool UAutoDestroySubsystem::RegisterActor(AActor* ActorToRegister)
{
	if (ActorToRegister && ActorToRegister->GetAutoDestroyWhenFinished())
	{
		ActorsToPoll.AddUnique(ActorToRegister);
		ActorToRegister->OnEndPlay.AddDynamic(this, &UAutoDestroySubsystem::OnActorEndPlay);
		return true;
	}

	return false;
}

void UAutoDestroySubsystem::OnActorEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason)
{
	UnregisterActor(Actor);
}

bool UAutoDestroySubsystem::UnregisterActor(AActor* ActorToRemove)
{
	if (ActorToRemove && ActorsToPoll.Remove(ActorToRemove))
	{
		ActorToRemove->OnEndPlay.RemoveAll(this);
		return true;
	}
	return false;
}

bool UAutoDestroySubsystem::CheckLatentActionsOnActor(FLatentActionManager& LatentActionManager, AActor* ActorToCheck, float WorldDeltaTime)
{
	UClass* const ActorClass = ActorToCheck ? ActorToCheck->GetClass() : nullptr;

	// Check if the latent action manager knows that this actor should be destroyed 	
	if (ActorClass && (ActorClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint) || !ActorClass->HasAnyClassFlags(CLASS_Native)))
	{
		return LatentActionManager.GetNumActionsForObject(ActorToCheck) == 0;
	}

	return true;
}

bool UAutoDestroySubsystem::ActorComponentsAreReadyForDestroy(AActor* const ActorToCheck)
{
	if (ActorToCheck)
	{
		// Check if this actors components are ready to be destroyed before marking it for auto destruction
		for (UActorComponent* const Comp : ActorToCheck->GetComponents())
		{
			if (Comp && !Comp->IsReadyForOwnerToAutoDestroy())
			{
				return false;
			}
		}
	}	

	return true;
}

ETickableTickType UAutoDestroySubsystem::GetTickableTickType() const
{
	// The CDO of this should never tick
	return IsTemplate() ? ETickableTickType::Never : ETickableTickType::Always;
}

void UAutoDestroySubsystem::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	UWorld* MyWorld = GetWorld();
	
	if (MyWorld && ActorsToPoll.Num())
	{
		// Use a copied array because if the user decides to override the OnDestroy of an actor and 
		// set it's AutoDestroy flag, we could end up modifiying the array
		TArray<AActor*, TInlineAllocator<8>> ActorsToDestroy;

		FLatentActionManager& LatentActionManager = MyWorld->GetLatentActionManager();

		for (int32 i = ActorsToPoll.Num() - 1; i >= 0; --i)
		{
			AActor* const CurActor = ActorsToPoll[i];

			if (CurActor && CurActor->GetAutoDestroyWhenFinished())
			{
				if (CheckLatentActionsOnActor(LatentActionManager, CurActor, DeltaTime) && ActorComponentsAreReadyForDestroy(CurActor))
				{
					ActorsToDestroy.Emplace(CurActor);
					ActorsToPoll.RemoveSwap(CurActor);
				}
			}
		}

		// Actually destroy all necessary actors
		for (AActor* CurActor : ActorsToDestroy)
		{
			if(CurActor)
			{
				CurActor->Destroy();
			}
		}
	}
}
