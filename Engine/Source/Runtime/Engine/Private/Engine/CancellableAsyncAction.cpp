// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/CancellableAsyncAction.h"
#include "Engine/GameInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CancellableAsyncAction)

void UCancellableAsyncAction::Cancel()
{
	// Child classes should override this
	SetReadyToDestroy();
}

void UCancellableAsyncAction::BeginDestroy()
{
	Cancel();

	Super::BeginDestroy();
}

bool UCancellableAsyncAction::IsActive() const
{
	return ShouldBroadcastDelegates();
}

bool UCancellableAsyncAction::ShouldBroadcastDelegates() const
{
	return IsRegistered();
}

bool UCancellableAsyncAction::IsRegistered() const
{
	return RegisteredWithGameInstance.IsValid();
}

class FTimerManager* UCancellableAsyncAction::GetTimerManager() const
{
	if (RegisteredWithGameInstance.IsValid())
	{
		return &RegisteredWithGameInstance->GetTimerManager();
	}

	return nullptr;
}

