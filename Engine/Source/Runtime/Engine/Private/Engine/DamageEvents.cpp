// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DamageEvents.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DamageEvents)

void FDamageEvent::GetBestHitInfo(AActor const* HitActor, AActor const* HitInstigator, FHitResult& OutHitInfo, FVector& OutImpulseDir) const
{
	if (ensure(HitActor))
	{
		// fill out the hitinfo as best we can
		OutHitInfo.HitObjectHandle = FActorInstanceHandle(const_cast<AActor*>(HitActor));
		OutHitInfo.bBlockingHit = true;
		OutHitInfo.BoneName = NAME_None;
		OutHitInfo.Component = Cast<UPrimitiveComponent>(HitActor->GetRootComponent());

		// assume the actor got hit in the center of its root component
		OutHitInfo.ImpactPoint = HitActor->GetActorLocation();
		OutHitInfo.Location = OutHitInfo.ImpactPoint;

		// assume hit came from instigator's location
		OutImpulseDir = HitInstigator ?
			(OutHitInfo.ImpactPoint - HitInstigator->GetActorLocation()).GetSafeNormal()
			: FVector::ZeroVector;

		// assume normal points back toward instigator
		OutHitInfo.ImpactNormal = -OutImpulseDir;
		OutHitInfo.Normal = OutHitInfo.ImpactNormal;
	}
}

void FPointDamageEvent::GetBestHitInfo(AActor const* HitActor, AActor const* HitInstigator, FHitResult& OutHitInfo, FVector& OutImpulseDir) const
{
	// assume the actor got hit in the center of its root component
	OutHitInfo = HitInfo;
	OutImpulseDir = ShotDirection;
}

void FRadialDamageEvent::GetBestHitInfo(AActor const* HitActor, AActor const* HitInstigator, FHitResult& OutHitInfo, FVector& OutImpulseDir) const
{
	ensure(ComponentHits.Num() > 0);

	// for now, just return the first one
	OutHitInfo = ComponentHits[0];
	OutImpulseDir = (OutHitInfo.ImpactPoint - Origin).GetSafeNormal();
}

float FRadialDamageParams::GetDamageScale(float DistanceFromEpicenter) const
{
	float const ValidatedInnerRadius = FMath::Max(0.f, InnerRadius);
	float const ValidatedOuterRadius = FMath::Max(OuterRadius, ValidatedInnerRadius);
	float const ValidatedDist = FMath::Max(0.f, DistanceFromEpicenter);

	if (ValidatedDist >= ValidatedOuterRadius)
	{
		// outside the radius, no effect
		return 0.f;
	}

	if ((DamageFalloff == 0.f) || (ValidatedDist <= ValidatedInnerRadius))
	{
		// no falloff or inside inner radius means full effect
		return 1.f;
	}

	// calculate the interpolated scale
	float DamageScale = 1.f - ((ValidatedDist - ValidatedInnerRadius) / (ValidatedOuterRadius - ValidatedInnerRadius));
	DamageScale = FMath::Pow(DamageScale, DamageFalloff);

	return DamageScale;
}

