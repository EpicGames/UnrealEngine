// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/EngineConsoleCommandExecutor.h"

#include "ConsoleSettings.h"
#include "Engine/GameEngine.h"
#include "Engine/LocalPlayer.h"
#include "Features/IModularFeatures.h"
#include "Framework/Commands/InputChord.h"

#define LOCTEXT_NAMESPACE "EngineConsoleCommandExecutor"

static const FLazyName GEngineConsoleCommandExecutorName = TEXT("Cmd");

FEngineConsoleCommandExecutor::FEngineConsoleCommandExecutor(UGameEngine* InGameEngine)
	: GameEngine(InGameEngine)
{
	IModularFeatures::Get().RegisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), this);
}

FEngineConsoleCommandExecutor::~FEngineConsoleCommandExecutor()
{
	IModularFeatures::Get().UnregisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), this);
}

FName FEngineConsoleCommandExecutor::StaticName()
{
	return GEngineConsoleCommandExecutorName;
}

FName FEngineConsoleCommandExecutor::GetName() const
{
	return StaticName();
}

FText FEngineConsoleCommandExecutor::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Cmd");
}

FText FEngineConsoleCommandExecutor::GetDescription() const
{
	return LOCTEXT("Description", "Execute Unreal Console Commands");
}

FText FEngineConsoleCommandExecutor::GetHintText() const
{
	return LOCTEXT("HintText", "Enter Console Command");
}

void FEngineConsoleCommandExecutor::GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out)
{
	const auto OnConsoleVariable = [&Out](const TCHAR* Name, IConsoleObject* CVar)
	{
		if (CVar->IsEnabled())
		{
			Out.Add(FConsoleSuggestion(Name, CVar->GetDetailedHelp().ToString()));
		}
	};

	IConsoleManager& ConsoleManager = IConsoleManager::Get();
	ConsoleManager.ForEachConsoleObjectThatContains(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), Input);
	for (const FString& CommandName : GetDefault<UConsoleSettings>()->GetFilteredManualAutoCompleteCommands(Input))
	{
		FString HelpString;
		// Try to find a console object for this entry in order to retrieve a help string if possible :
		const TCHAR* CommandNamePtr = *CommandName;
		if (IConsoleObject* CObj = ConsoleManager.FindConsoleObject(*FParse::Token(CommandNamePtr, /*UseEscape = */false), /*bTrackFrequentCalls = */false); CObj && CObj->IsEnabled())
		{
			HelpString = CObj->GetDetailedHelp().ToString();
		}
		Out.Add(FConsoleSuggestion(CommandName, HelpString));
	}
}

void FEngineConsoleCommandExecutor::GetExecHistory(TArray<FString>& Out)
{
	IConsoleManager::Get().GetConsoleHistory(TEXT(""), Out);
}

bool FEngineConsoleCommandExecutor::Exec(const TCHAR* Input)
{
	IConsoleManager::Get().AddConsoleHistoryEntry(TEXT(""), Input);

	int32 Len = FCString::Strlen(Input);
	TArray<TCHAR> Buffer;
	Buffer.AddZeroed(Len + 1);

	bool bHandled = false;
	while (FParse::Line(&Input, Buffer.GetData(), Buffer.Num()))
	{
		bHandled |= ExecInternal(Buffer.GetData());
	}
	return bHandled;
}

bool FEngineConsoleCommandExecutor::ExecInternal(const TCHAR* Input) const
{
	if (ULocalPlayer* Player = GameEngine->GetDebugLocalPlayer())
	{
		const bool bHandled = Player->Exec(Player->GetWorld(), Input, *GLog);
		if (bHandled)
		{
			return true;
		}
	}

	return GameEngine->Exec(GameEngine->GetGameWorld(), Input, *GLog);
}

bool FEngineConsoleCommandExecutor::AllowHotKeyClose() const
{
	return true;
}

bool FEngineConsoleCommandExecutor::AllowMultiLine() const
{
	return true;
}

FInputChord FEngineConsoleCommandExecutor::GetHotKey() const
{
	return {};
}

FInputChord FEngineConsoleCommandExecutor::GetIterateExecutorHotKey() const
{
	return {};
}

#undef LOCTEXT_NAMESPACE
