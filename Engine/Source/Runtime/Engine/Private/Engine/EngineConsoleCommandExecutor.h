// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

class UGameEngine;

class FEngineConsoleCommandExecutor final : public IConsoleCommandExecutor
{
public:
	explicit FEngineConsoleCommandExecutor(UGameEngine* GameEngine);
	~FEngineConsoleCommandExecutor();

	static FName StaticName();

	FName GetName() const final;
	FText GetDisplayName() const final;
	FText GetDescription() const final;
	FText GetHintText() const final;

	void GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out) final;
	void GetExecHistory(TArray<FString>& Out) final;
	bool Exec(const TCHAR* Input) final;

	bool AllowHotKeyClose() const final;
	bool AllowMultiLine() const final;
	FInputChord GetHotKey() const final;
	FInputChord GetIterateExecutorHotKey() const final;

private:
	bool ExecInternal(const TCHAR* Input) const;

	UGameEngine* GameEngine;
};
