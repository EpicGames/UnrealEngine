// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/EngineCustomTimeStep.h"

#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EngineCustomTimeStep)

void UEngineCustomTimeStep::UpdateApplicationLastTime()
{
	// Updates logical last time to match logical current time from last tick
	if (FMath::IsNearlyZero(FApp::GetLastTime()))
	{
		FApp::SetCurrentTime(FPlatformTime::Seconds() - 0.0001);
	}
	FApp::UpdateLastTime();
}

