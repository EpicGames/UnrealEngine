// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/HitResult.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HitResult)

FHitResult::FHitResult(class AActor* InActor, class UPrimitiveComponent* InComponent, FVector const& HitLoc, FVector const& HitNorm)
{
	FMemory::Memzero(this, sizeof(FHitResult));
	Location = HitLoc;
	ImpactPoint = HitLoc;
	Normal = HitNorm;
	ImpactNormal = HitNorm;
	HitObjectHandle = FActorInstanceHandle(InActor);
	Component = InComponent;
}

bool FHitResult::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	// Most of the time the vectors are the same values, use that as an optimization
	bool bImpactPointEqualsLocation = 0, bImpactNormalEqualsNormal = 0;

	// Often times the indexes are invalid, use that as an optimization
	bool bInvalidItem = 0, bInvalidFaceIndex = 0, bNoPenetrationDepth = 0, bInvalidElementIndex = 0;

	if (Ar.IsSaving())
	{
		bImpactPointEqualsLocation = (ImpactPoint == Location);
		bImpactNormalEqualsNormal = (ImpactNormal == Normal);
		bInvalidItem = (Item == INDEX_NONE);
		bInvalidFaceIndex = (FaceIndex == INDEX_NONE);
		bNoPenetrationDepth = (PenetrationDepth == 0.0f);
		bInvalidElementIndex = (ElementIndex == INDEX_NONE);
	}

	// pack bitfield with flags
	uint8 Flags = (bBlockingHit << 0) | (bStartPenetrating << 1) | (bImpactPointEqualsLocation << 2) | (bImpactNormalEqualsNormal << 3) | (bInvalidItem << 4) | (bInvalidFaceIndex << 5) | (bNoPenetrationDepth << 6) | (bInvalidElementIndex << 7);
	Ar.SerializeBits(&Flags, 8); 
	bBlockingHit = (Flags & (1 << 0)) ? 1 : 0;
	bStartPenetrating = (Flags & (1 << 1)) ? 1 : 0;
	bImpactPointEqualsLocation = (Flags & (1 << 2)) ? 1 : 0;
	bImpactNormalEqualsNormal = (Flags & (1 << 3)) ? 1 : 0;
	bInvalidItem = (Flags & (1 << 4)) ? 1 : 0;
	bInvalidFaceIndex = (Flags & (1 << 5)) ? 1 : 0;
	bNoPenetrationDepth = (Flags & (1 << 6)) ? 1 : 0;
	bInvalidElementIndex = (Flags & (1 << 7)) ? 1 : 0;
	// NOTE: Every bit in Flags is being used. If any more bits are added,
	// Flags will need to be upgraded to a uint16.

	Ar << Time;

	bOutSuccess = true;

	bool bOutSuccessLocal = true;

	Location.NetSerialize(Ar, Map, bOutSuccessLocal);
	bOutSuccess &= bOutSuccessLocal;
	Normal.NetSerialize(Ar, Map, bOutSuccessLocal);
	bOutSuccess &= bOutSuccessLocal;

	if (!bImpactPointEqualsLocation)
	{
		ImpactPoint.NetSerialize(Ar, Map, bOutSuccessLocal);
		bOutSuccess &= bOutSuccessLocal;
	}
	else if (Ar.IsLoading())
	{
		ImpactPoint = Location;
	}
	
	if (!bImpactNormalEqualsNormal)
	{
		ImpactNormal.NetSerialize(Ar, Map, bOutSuccessLocal);
		bOutSuccess &= bOutSuccessLocal;
	}
	else if (Ar.IsLoading())
	{
		ImpactNormal = Normal;
	}
	TraceStart.NetSerialize(Ar, Map, bOutSuccessLocal);
	bOutSuccess &= bOutSuccessLocal;
	TraceEnd.NetSerialize(Ar, Map, bOutSuccessLocal);
	bOutSuccess &= bOutSuccessLocal;

	if (!bNoPenetrationDepth)
	{
		Ar << PenetrationDepth;
	}
	else if(Ar.IsLoading())
	{
		PenetrationDepth = 0.0f;
	}

	if (Ar.IsLoading() && bOutSuccess)
	{
		Distance = (ImpactPoint - TraceStart).Size();
	}
	
	if (!bInvalidItem)
	{
		Ar << Item;
	}
	else if (Ar.IsLoading())
	{
		Item = INDEX_NONE;
	}

	Ar << PhysMaterial;

	if (Ar.IsLoading() && Ar.EngineNetVer() < FEngineNetworkCustomVersion::HitResultInstanceHandle)
	{
		AActor* HitActor = nullptr;
		Ar << HitActor;
		HitObjectHandle = HitActor;
	}
	else
	{
		Ar << HitObjectHandle;
	}
	Ar << Component;
	Ar << BoneName;
	if (!bInvalidFaceIndex)
	{
		Ar << FaceIndex;
	}
	else if (Ar.IsLoading())
	{
		FaceIndex = INDEX_NONE;
	}
	
	if (!bInvalidElementIndex)
	{
		Ar << ElementIndex;
	}
	else if (Ar.IsLoading())
	{
		ElementIndex = INDEX_NONE;
	}

	return true;
}

FString FHitResult::ToString() const
{
	return FString::Printf(TEXT("bBlockingHit:%s bStartPenetrating:%s Time:%f Location:%s ImpactPoint:%s Normal:%s ImpactNormal:%s TraceStart:%s TraceEnd:%s PenetrationDepth:%f Item:%d PhysMaterial:%s Actor:%s Component:%s BoneName:%s FaceIndex:%d"),
		bBlockingHit == true ? TEXT("True") : TEXT("False"),
		bStartPenetrating == true ? TEXT("True") : TEXT("False"),
		Time,
		*Location.ToString(),
		*ImpactPoint.ToString(),
		*Normal.ToString(),
		*ImpactNormal.ToString(),
		*TraceStart.ToString(),
		*TraceEnd.ToString(),
		PenetrationDepth,
		Item,
		PhysMaterial.IsValid() ? *PhysMaterial->GetName() : TEXT("None"),
		HitObjectHandle.IsValid() ? *HitObjectHandle.GetName() : TEXT("None"),
		Component.IsValid() ? *Component->GetName() : TEXT("None"),
		BoneName.IsValid() ? *BoneName.ToString() : TEXT("None"),
		FaceIndex);
}
