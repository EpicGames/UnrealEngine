// Copyright Epic Games, Inc. All Rights Reserved.
#include "Engine/HitResultNetSerializer.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(HitResultNetSerializer)


#include "Engine/HitResult.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"

namespace UE::Net
{

struct FHitResultNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasCustomNetReference = true;
	static constexpr bool bHasDynamicState = true; // Contains FNames which has dynamic state if serialized as strings

	// Types
	enum EReplicationFlags : uint32
	{
		BlockingHit = 1U,
		StartPenetrating = BlockingHit << 1U,
		ImpactPointEqualsLocation = StartPenetrating << 1U,
		ImpactNormalEqualsNormal = ImpactPointEqualsLocation << 1U,
		InvalidItem  = ImpactNormalEqualsNormal << 1U,
		InvalidFaceIndex = InvalidItem << 1U,
		NoPenetrationDepth = InvalidFaceIndex << 1U,
		InvalidElementIndex = NoPenetrationDepth << 1U,
	};

	static constexpr uint32 ReplicatedFlagCount = 8U;

	struct FQuantizedType
	{
		alignas(16) uint8 HitResult[316];
		uint32 ReplicationFlags;
	};

	typedef FHitResult SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FHitResultNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	// Conditional properties
	enum EPropertyName : uint32
	{
		PropertyName_FaceIndex,
		PropertyName_Distance,
		PropertyName_ImpactPoint,
		PropertyName_ImpactNormal,
		PropertyName_PenetrationDepth,
		PropertyName_Item,
		PropertyName_ElementIndex,
		PropertyName_bBlockingHit,
		PropertyName_bStartPenetrating,

		PropertyName_ConditionallyReplicatedPropertyCount
	};

	static FNetBitArrayView GetMemberChangeMask(uint32* MemberMaskStorage, uint32 MemberCount, uint32 ReplicationFlags);

	static uint16 PropertyToMemberIndex[EPropertyName::PropertyName_ConditionallyReplicatedPropertyCount];

	static FHitResultNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static FStructNetSerializerConfig StructNetSerializerConfigForHitResult;
	static const FNetSerializer* StructNetSerializer;
	static UE::Net::EReplicationStateTraits HitResultStateTraits;
};

UE_NET_IMPLEMENT_SERIALIZER(FHitResultNetSerializer);

const FHitResultNetSerializer::ConfigType FHitResultNetSerializer::DefaultConfig;
FHitResultNetSerializer::FNetSerializerRegistryDelegates FHitResultNetSerializer::NetSerializerRegistryDelegates;
FStructNetSerializerConfig FHitResultNetSerializer::StructNetSerializerConfigForHitResult;
const FNetSerializer* FHitResultNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);
// Populated after generating a ReplicationStateDescriptor for this struct
uint16 FHitResultNetSerializer::PropertyToMemberIndex[FHitResultNetSerializer::EPropertyName::PropertyName_ConditionallyReplicatedPropertyCount];
EReplicationStateTraits FHitResultNetSerializer::HitResultStateTraits;

FNetBitArrayView FHitResultNetSerializer::GetMemberChangeMask(uint32* MemberMaskStorage, uint32 MemberCount, uint32 ReplicationFlags)
{
	FNetBitArrayView MemberMask(MemberMaskStorage, MemberCount, FNetBitArrayView::NoResetNoValidate);

	// Setup conditional properties
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_ImpactPoint], (ReplicationFlags & EReplicationFlags::ImpactPointEqualsLocation) == 0);
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_ImpactNormal], (ReplicationFlags & EReplicationFlags::ImpactNormalEqualsNormal) == 0);
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_PenetrationDepth], (ReplicationFlags & EReplicationFlags::NoPenetrationDepth) == 0);
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_Item], (ReplicationFlags & EReplicationFlags::InvalidItem) == 0);
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_FaceIndex], (ReplicationFlags & EReplicationFlags::InvalidFaceIndex) == 0);
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_ElementIndex], (ReplicationFlags & EReplicationFlags::InvalidElementIndex) == 0);

	// Distance is calculated
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_Distance], false);

	// Never replicate the bools since they are included in the flag field
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_bBlockingHit], false);
	MemberMask.SetBitValue(PropertyToMemberIndex[EPropertyName::PropertyName_bStartPenetrating], false);

	return MemberMask;
}

void FHitResultNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const uint32 ReplicationFlags = Value.ReplicationFlags;
	Writer->WriteBits(ReplicationFlags, FHitResultNetSerializer::ReplicatedFlagCount);

	// We need to manually serialize the properties as there are some replicated properties that don't need to be replicated depending on the data
	// as is the case with some of the bools that are covered by the ReplicationFlags.

	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForHitResult.StateDescriptor.GetReference();
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;

	const uint32 MemberCount = Descriptor->MemberCount;

	// Initalize mask as all dirty		
	uint32 MemberMaskStorage = ~0U;
	FNetBitArrayView MemberMask = GetMemberChangeMask(&MemberMaskStorage, MemberCount, ReplicationFlags);

	for (uint32 MemberIndex = 0; MemberIndex < MemberCount; ++MemberIndex)
	{
		if (!MemberMask.GetBit(MemberIndex))
		{
			continue;
		}

		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIndex];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIndex];

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIndex].DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		FNetSerializeArgs MemberSerializeArgs;
		MemberSerializeArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberSerializeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Value.HitResult) + MemberDescriptor.InternalMemberOffset;
		MemberSerializerDescriptor.Serializer->Serialize(Context, MemberSerializeArgs);
	}
}

void FHitResultNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const uint32 ReplicationFlags = Reader->ReadBits(FHitResultNetSerializer::ReplicatedFlagCount);
	Target.ReplicationFlags = ReplicationFlags;

	// We need to manually serialize the properties as there are some replicated properties that don't need to be replicated depending on the data
	// as is the case with some of the bools that are covered by the ReplicationFlags.
	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForHitResult.StateDescriptor.GetReference();
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;

	const uint32 MemberCount = Descriptor->MemberCount;		

	// Initalize mask as all dirty		
	uint32 MemberMaskStorage = ~0U;
	FNetBitArrayView MemberMask = GetMemberChangeMask(&MemberMaskStorage, MemberCount, ReplicationFlags);

	for (uint32 MemberIndex = 0; MemberIndex < MemberCount; ++MemberIndex)
	{
		if (!MemberMask.GetBit(MemberIndex))
		{
			continue;
		}

		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIndex];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIndex];

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIndex].DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		FNetDeserializeArgs MemberDeserializeArgs;
		MemberDeserializeArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberDeserializeArgs.Target = reinterpret_cast<NetSerializerValuePointer>(&Target.HitResult) + MemberDescriptor.InternalMemberOffset;
		MemberSerializerDescriptor.Serializer->Deserialize(Context, MemberDeserializeArgs);
	}
}

void FHitResultNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	Serialize(Context, Args);
}

void FHitResultNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	Deserialize(Context, Args);
}

void FHitResultNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	uint32 ReplicationFlags = 0;

	// Update flags based on SourceValue
	ReplicationFlags |= SourceValue.ImpactPoint == SourceValue.Location ? EReplicationFlags::ImpactPointEqualsLocation : 0U;
	ReplicationFlags |= SourceValue.ImpactNormal == SourceValue.Normal ? EReplicationFlags::ImpactNormalEqualsNormal : 0U;
	ReplicationFlags |= SourceValue.Item == INDEX_NONE ? EReplicationFlags::InvalidItem : 0U;
	ReplicationFlags |= SourceValue.FaceIndex == INDEX_NONE ? EReplicationFlags::InvalidFaceIndex : 0U;
	ReplicationFlags |= (SourceValue.PenetrationDepth == 0.0f) ? EReplicationFlags::NoPenetrationDepth : 0U;
	ReplicationFlags |= SourceValue.ElementIndex == INDEX_NONE ? EReplicationFlags::InvalidElementIndex : 0U;
	ReplicationFlags |= SourceValue.bBlockingHit ? EReplicationFlags::BlockingHit : 0U;
	ReplicationFlags |= SourceValue.bStartPenetrating ? EReplicationFlags::StartPenetrating : 0U;
	
	TargetValue.ReplicationFlags = ReplicationFlags;

	// We do a full quantize even though we wont necessarily serialize them.
	FNetQuantizeArgs HitResultQuantizeArgs = {};
	HitResultQuantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
	HitResultQuantizeArgs.Source = Args.Source;
	HitResultQuantizeArgs.Target = NetSerializerValuePointer(&TargetValue.HitResult);
	StructNetSerializer->Quantize(Context, HitResultQuantizeArgs);
}

void FHitResultNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);

	const uint32 ReplicatonFlags = SourceValue.ReplicationFlags;
	
	// Dequantize all and fixup afterwards
	FNetDequantizeArgs HitResultQuantizeArgs = {};
	HitResultQuantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
	HitResultQuantizeArgs.Source = NetSerializerValuePointer(&SourceValue.HitResult);
	HitResultQuantizeArgs.Target = Args.Target;
	StructNetSerializer->Dequantize(Context, HitResultQuantizeArgs);

	if (ReplicatonFlags & EReplicationFlags::ImpactPointEqualsLocation)
	{
		TargetValue.ImpactPoint = TargetValue.Location;
	}
	
	if (ReplicatonFlags & EReplicationFlags::ImpactNormalEqualsNormal)
	{
		TargetValue.ImpactNormal = TargetValue.Normal;
	}

	if (ReplicatonFlags & EReplicationFlags::InvalidItem)
	{
		TargetValue.Item = INDEX_NONE;
	}

	if (ReplicatonFlags & EReplicationFlags::InvalidFaceIndex)
	{
		TargetValue.FaceIndex = INDEX_NONE;
	}

	if (ReplicatonFlags & EReplicationFlags::NoPenetrationDepth)
	{
		TargetValue.PenetrationDepth = 0.f;
	}

	if (ReplicatonFlags & EReplicationFlags::InvalidElementIndex)
	{
		TargetValue.ElementIndex = INDEX_NONE;
	}

	// Calculate distance
	TargetValue.Distance = (TargetValue.ImpactPoint - TargetValue.TraceStart).Size();

	// Set the bBlockingHit and bStartPenetrating from the flags
	TargetValue.bBlockingHit = (ReplicatonFlags & EReplicationFlags::BlockingHit) != 0U ? 1 : 0;
	TargetValue.bStartPenetrating = (ReplicatonFlags & EReplicationFlags::StartPenetrating) != 0U ? 1 : 0;
}

void FHitResultNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	
	FNetCollectReferencesArgs HitResultCollectReferencesArgs = Args;
	HitResultCollectReferencesArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
	HitResultCollectReferencesArgs.Source = NetSerializerValuePointer(&Value.HitResult);
	StructNetSerializer->CollectNetReferences(Context, HitResultCollectReferencesArgs);
}

bool FHitResultNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Value0.ReplicationFlags != Value1.ReplicationFlags)
		{
			return false;
		}

		// Do a per member compare of relevant members
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForHitResult.StateDescriptor.GetReference();
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
		const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;

		const uint32 MemberCount = Descriptor->MemberCount;
		
		// Initalize mask as all dirty		
		uint32 MemberMaskStorage = ~0U;
		FNetBitArrayView MemberMask = GetMemberChangeMask(&MemberMaskStorage, MemberCount, Value0.ReplicationFlags);

		FNetIsEqualArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.bStateIsQuantized = Args.bStateIsQuantized;
		
		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			if (!MemberMask.GetBit(MemberIt))
			{
				continue;
			}

			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
			const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

			MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			const uint32 MemberOffset = MemberDescriptor.InternalMemberOffset;
			MemberArgs.Source0 = reinterpret_cast<NetSerializerValuePointer>(&Value0.HitResult) + MemberDescriptor.InternalMemberOffset;
			MemberArgs.Source1 = reinterpret_cast<NetSerializerValuePointer>(&Value1.HitResult) + MemberDescriptor.InternalMemberOffset;

			if (!Serializer->IsEqual(Context, MemberArgs))
			{
				return false;
			}
		}
	}
	else
	{
		FNetIsEqualArgs HitResultEqualArgs = Args;
		HitResultEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitResultEqualArgs.Source0 = NetSerializerValuePointer(Args.Source0);
		HitResultEqualArgs.Source1 = NetSerializerValuePointer(Args.Source1);

		if (!StructNetSerializer->IsEqual(Context, HitResultEqualArgs))
		{
			return false;
		}
	}

	return true;
}

bool FHitResultNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);

	FNetValidateArgs HitResultValidateArgs = {};
	HitResultValidateArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
	HitResultValidateArgs.Source = Args.Source;
	if (!StructNetSerializer->Validate(Context, HitResultValidateArgs))
	{
		return false;
	}

	return true;
}

void FHitResultNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	if (EnumHasAnyFlags(HitResultStateTraits, UE::Net::EReplicationStateTraits::HasDynamicState))
	{
		FNetCloneDynamicStateArgs HitResultCloneDynamicStateArgs = {};
		HitResultCloneDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitResultCloneDynamicStateArgs.Source = NetSerializerValuePointer(&SourceValue.HitResult);
		HitResultCloneDynamicStateArgs.Target = NetSerializerValuePointer(&TargetValue.HitResult);
		StructNetSerializer->CloneDynamicState(Context, HitResultCloneDynamicStateArgs);
	}
}

void FHitResultNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	
	if (EnumHasAnyFlags(HitResultStateTraits, UE::Net::EReplicationStateTraits::HasDynamicState))
	{
		FNetFreeDynamicStateArgs HitResultFreeDynamicStateArgs = {};
		HitResultFreeDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitResultFreeDynamicStateArgs.Source = NetSerializerValuePointer(&SourceValue.HitResult);
		StructNetSerializer->FreeDynamicState(Context, HitResultFreeDynamicStateArgs);
	}
}

static const FName PropertyNetSerializerRegistry_NAME_HitResult("HitResult");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_HitResult, FHitResultNetSerializer);

FHitResultNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_HitResult);
}

void FHitResultNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_HitResult);
}

void FHitResultNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	// Setup serializer
	{
		// In this case we want to build a descriptor based on the struct members rather than the serializer we try to register
		FReplicationStateDescriptorBuilder::FParameters Params;
		Params.SkipCheckForCustomNetSerializerForStruct = true;

		const UStruct* HitResultStruct = FHitResult::StaticStruct();

		// Had do comment this out as the size differs between editor and non-editor builds.
		//if (HitResultStruct->GetStructureSize() != 240 || HitResultStruct->GetMinAlignment() != 8)
		//{
		//	LowLevelFatalError(TEXT("%s Size: %d Alignment: %d"), TEXT("FHitResult layout has changed. Need to update FHitResultNetSerializer."), HitResultStruct->GetStructureSize(), HitResultStruct->GetMinAlignment());
		//}

		StructNetSerializerConfigForHitResult.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(HitResultStruct, Params);
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForHitResult.StateDescriptor.GetReference();
		check(Descriptor != nullptr);
		HitResultStateTraits = Descriptor->Traits;

		if (Descriptor->MemberCount > 32U)
		{
			LowLevelFatalError(TEXT("FHitResultNetSerializer has more than 32 replicated members (%u) needs to be updated."), Descriptor->MemberCount);
		}
	
		// Build property -> Member index lookup table
		FName PropertyNames[EPropertyName::PropertyName_ConditionallyReplicatedPropertyCount];

		PropertyNames[PropertyName_FaceIndex] =	FName("FaceIndex");
		PropertyNames[PropertyName_Distance] =	FName("Distance");
		PropertyNames[PropertyName_ImpactPoint] = FName("ImpactPoint");
		PropertyNames[PropertyName_ImpactNormal] = FName("ImpactNormal");
		PropertyNames[PropertyName_PenetrationDepth] = FName("PenetrationDepth");
		PropertyNames[PropertyName_Item] = FName("Item");
		PropertyNames[PropertyName_ElementIndex] = FName("ElementIndex");
		PropertyNames[PropertyName_bBlockingHit] = FName("bBlockingHit");
		PropertyNames[PropertyName_bStartPenetrating] = FName("bStartPenetrating");

		uint32 FoundPropertyMask = 0;

		// Find all replicated properties of interest.
		const FProperty*const*MemberProperties = Descriptor->MemberProperties;
		for (uint32 PropertyIndex = 0; PropertyIndex != EPropertyName::PropertyName_ConditionallyReplicatedPropertyCount; ++PropertyIndex)
		{
			for (const FProperty*const& MemberProperty : MakeArrayView(MemberProperties, Descriptor->MemberCount))
			{
				const SIZE_T MemberIndex = &MemberProperty - MemberProperties;
				if (MemberProperty->GetFName() == PropertyNames[PropertyIndex])
				{
					FoundPropertyMask |= 1U << PropertyIndex;

					FHitResultNetSerializer::PropertyToMemberIndex[PropertyIndex] = MemberIndex;
					break;
				}
			}
		}

		if (FoundPropertyMask != (1U <<  EPropertyName::PropertyName_ConditionallyReplicatedPropertyCount) - 1U)
		{
			LowLevelFatalError(TEXT("%s"), TEXT("Couldn't find expected replicated members in FHitResult.")); 
		}

		// Validate our assumptions regarding quantized state size and alignment.
		constexpr SIZE_T OffsetOfHitResult = offsetof(FQuantizedType, HitResult);
		if ((sizeof(FQuantizedType::HitResult) < Descriptor->InternalSize) || (((OffsetOfHitResult/Descriptor->InternalAlignment)*Descriptor->InternalAlignment) != OffsetOfHitResult))
		{
			LowLevelFatalError(TEXT("FQuantizedType::HitResult has size %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType::HitResult)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment));
		}
	}

	// Verify traits
	ValidateForwardingNetSerializerTraits(&UE_NET_GET_SERIALIZER(FHitResultNetSerializer), HitResultStateTraits);
}

}


