// Copyright Epic Games, Inc. All Rights Reserved.

#include "LODSyncInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LODSyncInterface)

ULODSyncInterface::ULODSyncInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

