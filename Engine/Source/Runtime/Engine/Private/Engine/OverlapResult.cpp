// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/OverlapResult.h"

#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OverlapResult)

int32 FOverlapResult::GetItemIndexInternal() const
{
	return ItemIndex;
}

int32 FOverlapResult::GetItemIndex() const
{
	if (const IPhysicsBodyInstanceOwner* BodyInstanceOwner = IPhysicsBodyInstanceOwner::GetPhysicsBodyInstanceOwnerFromOverlapResult(*this))
	{
		return BodyInstanceOwner->IsMultiBodyOverlap() ? ItemIndex : INDEX_NONE;
	}
	return INDEX_NONE;
}

void FOverlapResult::SetItemIndex(const int32 InItemIndex)
{
	ItemIndex = InItemIndex;
}
