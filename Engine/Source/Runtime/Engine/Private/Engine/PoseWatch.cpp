// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/PoseWatch.h"
#include "Animation/AnimBlueprint.h"
#include "Textures/SlateIcon.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseWatch)

#if WITH_EDITOR
#include "AnimationEditorUtils.h"
#endif

#define LOCTEXT_NAMESPACE "PoseWatch"

#if WITH_EDITOR

// PoseWatchUtil

TSet<UPoseWatch*> PoseWatchUtil::GetChildrenPoseWatchOf(const UPoseWatchFolder* Folder, const UAnimBlueprint* AnimBlueprint)
{
	TSet<UPoseWatch*> Children;

	for (UPoseWatch* SomePoseWatch : AnimBlueprint->PoseWatches)
	{
		if (SomePoseWatch->IsIn(Folder))
		{
			Children.Add(SomePoseWatch);
		}
	}

	return Children;
}

TSet<UPoseWatchFolder*> PoseWatchUtil::GetChildrenPoseWatchFoldersOf(const UPoseWatchFolder* Folder, const UAnimBlueprint* AnimBlueprint)
{
	TSet<UPoseWatchFolder*> Children;

	for (UPoseWatchFolder* SomePoseWatchFolder : AnimBlueprint->PoseWatchFolders)
	{
		if (SomePoseWatchFolder->IsIn(Folder))
		{
			Children.Add(SomePoseWatchFolder);
		}
	}

	return Children;
}

template <typename TParent, typename TItem> FText PoseWatchUtil::FindUniqueNameInParent(TParent* InParent, const TItem* InItem)
{
	static const FTextFormat LabelFormat = FTextFormat::FromString("{0}{1}");

	FText NewLabel = InItem->GetLabel();

	for (uint32 Counter = 1; !InItem->IsLabelUniqueInParent(NewLabel, InParent); ++Counter)
	{
		NewLabel = FText::Format(LabelFormat, InItem->GetLabel(), FText::AsNumber(Counter));
	}

	return NewLabel;
}

FColor PoseWatchUtil::ChoosePoseWatchColor()
{
	return FColor::MakeRandomColor();
}

#endif // WITH_EDITOR

// UPoseWatchFolder

UPoseWatchFolder::UPoseWatchFolder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Label = GetDefaultLabel();
#endif
}

#if WITH_EDITOR

const FText UPoseWatchFolder::GetPath() const
{
	if (Parent.Get() != nullptr)
	{
		return FText::Format(LOCTEXT("Path", "{0}/{1}"), Parent->GetPath(), Label);
	}
	return Label;
}

FText UPoseWatchFolder::GetDefaultLabel() const
{
	return LOCTEXT("PoseWatchFolderDefaultName", "NewFolder");
}

FText UPoseWatchFolder::GetLabel() const
{
	return Label;
}

bool UPoseWatchFolder::GetIsVisible() const
{
	return bIsVisible;
}

UPoseWatchFolder* UPoseWatchFolder::GetParent() const
{
	return Parent.Get();
}

bool UPoseWatchFolder::SetParent(UPoseWatchFolder* InParent, bool bForce)
{
	if (IsLabelUniqueInParent(Label, InParent))
	{
		Parent = InParent;
		return true;
	}
	else if (bForce)
	{
		Label = FindUniqueNameInParent(InParent);
		check(IsLabelUniqueInParent(Label, InParent))
		Parent = InParent;
		return true;
	}
	return false;
}

bool UPoseWatchFolder::IsLabelUniqueInParent(const FText& InLabel, UPoseWatchFolder* InFolder) const
{
	for (UPoseWatchFolder* SomeChildFolder : PoseWatchUtil::GetChildrenPoseWatchFoldersOf(InFolder, GetAnimBlueprint()))
	{
		if (SomeChildFolder->GetLabel().ToString().Equals(InLabel.ToString()))
		{
			if (SomeChildFolder != this)
			{
				return false;
			}
		}
	}
	return true;

	// TODO - folders can contain folders AND node watches ... don't we need to check against node watch names as well ?
}

void UPoseWatchFolder::MoveTo(UPoseWatchFolder* InFolder)
{
	SetParent(InFolder);
}

bool UPoseWatchFolder::SetLabel(const FText& InLabel)
{
	FText NewLabel = FText::TrimPrecedingAndTrailing(InLabel);
	if (IsLabelUniqueInParent(NewLabel, Parent.Get()))
	{
		Label = NewLabel;
		return true;
	}
	return false;
}

void UPoseWatchFolder::SetIsVisible(bool bInIsVisible, bool bUpdateChildren)
{
	// Can only become visible if there are no children descendents
	if (!HasPoseWatchDescendents() && bInIsVisible)
	{
		bIsVisible = false;
		return;
	}

	bIsVisible = bInIsVisible;

	if (bUpdateChildren)
	{
		for (UPoseWatch* SomePoseWatch : PoseWatchUtil::GetChildrenPoseWatchOf(this, GetAnimBlueprint()))
		{
			SomePoseWatch->SetIsVisible(bInIsVisible);
		}
		for (UPoseWatchFolder* SomeChildFolder : PoseWatchUtil::GetChildrenPoseWatchFoldersOf(this, GetAnimBlueprint()))
		{
			SomeChildFolder->SetIsVisible(bInIsVisible);
		}
	}
}

void UPoseWatchFolder::SetIsExpanded(bool bInIsExpanded)
{
	bIsExpanded = bInIsExpanded;
}

bool UPoseWatchFolder::GetIsExpanded() const
{
	return bIsExpanded;
}

void UPoseWatchFolder::OnRemoved()
{
	// Move all this folder's children to this folder's parent
	for (UPoseWatch* SomePoseWatch : PoseWatchUtil::GetChildrenPoseWatchOf(this, GetAnimBlueprint()))
	{
		SomePoseWatch->SetParent(Parent.Get(), /* bForce*/ true);
	}
	for (UPoseWatchFolder* SomePoseWatchFolder : PoseWatchUtil::GetChildrenPoseWatchFoldersOf(this, GetAnimBlueprint()))
	{
		SomePoseWatchFolder->SetParent(Parent.Get(), /* bForce*/ true);
	}

	UAnimBlueprint* AnimBlueprint = CastChecked<UAnimBlueprint>(GetOuter());
	AnimBlueprint->PoseWatchFolders.Remove(this);

	if (Parent.IsValid())
	{
		Parent->UpdateVisibility();
	}

	AnimationEditorUtils::OnPoseWatchesChanged().Broadcast(GetAnimBlueprint(), nullptr);
}

void UPoseWatchFolder::UpdateVisibility()
{
	bool bNewIsVisible = false;

	for (UPoseWatch* SomePoseWatch : PoseWatchUtil::GetChildrenPoseWatchOf(this, GetAnimBlueprint()))
	{
		bNewIsVisible |= SomePoseWatch->GetIsVisible();
	}
	for (UPoseWatchFolder* SomePoseWatchFolder : PoseWatchUtil::GetChildrenPoseWatchFoldersOf(this, GetAnimBlueprint()))
	{
		bNewIsVisible |= SomePoseWatchFolder->GetIsVisible();
	}

	SetIsVisible(bNewIsVisible, false);
	if (Parent.IsValid())
	{
		Parent->UpdateVisibility();
	}
}

UAnimBlueprint* UPoseWatchFolder::GetAnimBlueprint() const
{
	return  CastChecked<UAnimBlueprint>(GetOuter());
}

bool UPoseWatchFolder::IsIn(const UPoseWatchFolder* InFolder) const
{
	return Parent.Get() == InFolder;
}

bool UPoseWatchFolder::IsDescendantOf(const UPoseWatchFolder* InFolder) const
{
	if (IsIn(InFolder))
	{
		return true;
	}

	TWeakObjectPtr<UPoseWatchFolder> ParentFolder = Parent;
	while (ParentFolder.IsValid())
	{
		if (ParentFolder->IsIn(InFolder))
		{
			return true;
		}
		ParentFolder = ParentFolder->Parent;
	}
	return false;
}

bool UPoseWatchFolder::IsAssignedFolder() const
{
	return Parent != nullptr;
}

bool UPoseWatchFolder::ValidateLabelRename(const FText& InLabel, FText& OutErrorMessage)
{
	FText UseLabel = FText::TrimPrecedingAndTrailing(InLabel);
	if (UseLabel.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("PoseWatchFolderNameEmpty", "A pose watch folder must have a label");
		return false;
	}

	if (!IsLabelUniqueInParent(UseLabel, Parent.Get()))
	{
		OutErrorMessage = LOCTEXT("PoseWatchFolderNameTaken", "A folder already has this name at this level");
		return false;
	}
	return true;
}

bool UPoseWatchFolder::HasChildren() const
{
	if (PoseWatchUtil::GetChildrenPoseWatchFoldersOf(this, GetAnimBlueprint()).Num() > 0)
	{
		return true;
	}
	if (PoseWatchUtil::GetChildrenPoseWatchOf(this, GetAnimBlueprint()).Num() > 0)
	{
		return true;
	}
	return false;
}

void UPoseWatchFolder::SetUniqueDefaultLabel()
{
	Label = GetDefaultLabel();
	Label = FindUniqueNameInParent(Parent.Get());
}

FText UPoseWatchFolder::FindUniqueNameInParent(UPoseWatchFolder* InFolder) const
{
	return PoseWatchUtil::FindUniqueNameInParent(InFolder, this);
}

bool UPoseWatchFolder::HasPoseWatchChildren() const
{
	return PoseWatchUtil::GetChildrenPoseWatchOf(this, GetAnimBlueprint()).Num() > 0;
}

bool UPoseWatchFolder::HasPoseWatchDescendents() const
{
	if (HasPoseWatchChildren())
	{
		return true;
	}
	for (UPoseWatchFolder* SomePoseWatchFolder : PoseWatchUtil::GetChildrenPoseWatchFoldersOf(this, GetAnimBlueprint()))
	{
		if (SomePoseWatchFolder->HasPoseWatchDescendents())
		{
			return true;
		}
	}
	return false;
}

// UPoseWatchElement

FText UPoseWatchElement::GetDefaultLabel() const
{
	return LOCTEXT("NewPoseWatchElement", "Pose Watch");
}

FText UPoseWatchElement::GetLabel() const
{
	return Label;
}

bool UPoseWatchElement::GetIsVisible() const
{
	return bIsVisible;
}

bool UPoseWatchElement::GetIsEnabled() const
{
	return true;
}

UPoseWatch* UPoseWatchElement::GetParent() const
{
	return Parent.Get();
}

void UPoseWatchElement::SetParent(UPoseWatch* InParent)
{
	Parent = InParent;
}

bool UPoseWatchElement::SetLabel(const FText& InLabel)
{
	if (Parent.IsValid() && IsLabelUniqueInParent(InLabel, Parent.Get()))
	{
		Label = InLabel;
		return true;
	}

	return false;
}

void UPoseWatchElement::SetIsVisible(bool bInIsVisible)
{
	bIsVisible = bInIsVisible;

	if (Parent.IsValid())
	{
		Parent->UpdateVisibility();
	}
}

bool UPoseWatchElement::ValidateLabelRename(const FText& InLabel, FText& OutErrorMessage)
{
	if (!IsLabelUniqueInParent(InLabel, Parent.Get()))
	{
		OutErrorMessage = LOCTEXT("PoseWatchElementNameTaken", "A element already has this name in this node watch");
		return false;
	}

	return true;
}

bool UPoseWatchElement::IsLabelUniqueInParent(const FText& InLabel, UPoseWatch* InParent) const
{
	for (TObjectPtr<UPoseWatchElement>& Element : InParent->GetElements())
	{
		if (Element->GetLabel().ToString().Equals(InLabel.ToString()))
		{
			if (Element != this)
			{
				return false;
			}
		}
	}
	return true;
}

void UPoseWatchElement::SetUniqueDefaultLabel()
{
	Label = GetDefaultLabel();
	Label = FindUniqueNameInParent(Parent.Get());
}

void UPoseWatchElement::SetUniqueLabel(const FText& InLabel)
{
	Label = InLabel;
	Label = FindUniqueNameInParent(Parent.Get());
}

FColor UPoseWatchElement::GetColor() const
{
	return Color;
}

void UPoseWatchElement::SetColor(const FColor& InColor)
{
	Color = InColor;
}

bool UPoseWatchElement::HasColor() const
{
	return bHasColor;
}

void UPoseWatchElement::SetHasColor(const bool bInHasColor)
{
	bHasColor = bInHasColor;
}

FName UPoseWatchElement::GetIconName() const
{
	return IconName;
}

void UPoseWatchElement::SetIconName(const FName InIconName)
{
	IconName = InIconName;
}

void UPoseWatchElement::ToggleIsVisible()
{
	SetIsVisible(!bIsVisible);
}

FText UPoseWatchElement::FindUniqueNameInParent(UPoseWatch* InParent) const
{
	if (InParent)
	{
		return PoseWatchUtil::FindUniqueNameInParent(InParent, this);
	}
	return FText();
}

#endif // WITH_EDITOR

// UPoseWatchPoseElement

UPoseWatchPoseElement::UPoseWatchPoseElement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	ViewportMask = nullptr;
	bInvertViewportMask = false;
	BlendScaleThreshold = 0.f;
	ViewportOffset = FVector3d::ZeroVector;

	Label = GetDefaultLabel();
	SetIconName("AnimGraph.PoseWatch.Icon");
#endif
}

#if WITH_EDITOR

bool UPoseWatchPoseElement::GetIsEnabled() const
{
	return Parent.Get() && Parent->GetIsNodeEnabled();
}

FSlateIcon UPoseWatchPoseElement::StaticGetIcon()
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "AnimGraph.PoseWatch.Icon");
	return Icon;
}

#endif // WITH_EDITOR

// UPoseWatch

UPoseWatch::UPoseWatch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Label = GetDefaultLabel();
#endif
}

#if WITH_EDITOR

void UPoseWatch::SetIsExpanded(bool bInIsExpanded)
{
	bIsExpanded = bInIsExpanded;
}

bool UPoseWatch::GetIsExpanded() const
{
	return bIsExpanded;
}

const FText UPoseWatch::GetPath() const
{
	check(!Label.IsEmpty())
	if (Parent.Get() != nullptr)
	{
		return FText::Format(LOCTEXT("Path", "{0}/{1}"), Parent->GetPath(), Label);
	}
	return Label;
}

FText UPoseWatch::GetLabel() const
{
	return Label;
}

FText UPoseWatch::GetDefaultLabel() const
{
	if (Node.IsValid())
	{
		return Node->GetNodeTitle(ENodeTitleType::ListView);
	}
	return LOCTEXT("NewPoseWatch", "NewPoseWatch");
}

bool UPoseWatch::GetIsVisible() const
{
	return bIsVisible;
}

bool UPoseWatch::GetIsNodeEnabled() const
{
	return bIsNodeEnabled;
}

bool UPoseWatch::GetIsEnabled() const
{
 	bool bIsEnabled = false;
  
	// Set visible if any child component is visible.
	for (const TObjectPtr<UPoseWatchElement>& PoseWatchElement : Elements)
	{
  		bIsEnabled |= PoseWatchElement->GetIsEnabled();
	}
  
	return bIsEnabled;
}

FColor UPoseWatch::GetColor() const
{
	return Color_DEPRECATED;
}

bool UPoseWatch::GetShouldDeleteOnDeselect() const
{
	return bDeleteOnDeselection;
}

void UPoseWatch::OnRemoved()
{
	UAnimBlueprint* AnimBlueprint = CastChecked<UAnimBlueprint>(GetOuter());
	AnimBlueprint->PoseWatches.Remove(this);
	
	AnimationEditorUtils::RemovePoseWatch(this, AnimBlueprint);

	if (Parent.IsValid())
	{
		Parent->UpdateVisibility();
	}

	AnimationEditorUtils::OnPoseWatchesChanged().Broadcast(GetAnimBlueprint(), Node.Get());
}

UPoseWatchFolder* UPoseWatch::GetParent() const
{
	return Parent.Get();
}

bool UPoseWatch::SetParent(UPoseWatchFolder* InParent, bool bForce)
{
	if (!IsLabelUniqueInParent(Label, InParent))
	{
		if (!bForce)
		{
			return false;
		}
		Label = FindUniqueNameInParent(InParent);
	}

	UPoseWatchFolder* OldParent = Parent.Get();
	Parent = InParent;

	if (OldParent)
	{
		OldParent->UpdateVisibility();
	}

	if (InParent)
	{
		InParent->UpdateVisibility();
		InParent->SetIsExpanded(true);
	}

	return true;
}

void UPoseWatch::SetIsNodeEnabled(const bool bInIsNodeEnabled)
{
	bIsNodeEnabled = bInIsNodeEnabled;
}

void UPoseWatch::MoveTo(UPoseWatchFolder* InFolder)
{
	SetParent(InFolder);
}

bool UPoseWatch::SetLabel(const FText& InLabel)
{
	FText NewLabel = FText::TrimPrecedingAndTrailing(InLabel);

	if (IsLabelUniqueInParent(NewLabel, Parent.Get()))
	{
		Label = NewLabel;
		return true;
	}

	return false;
}

void UPoseWatch::SetIsVisible(bool bInIsVisible)
{
	for (const TObjectPtr<UPoseWatchElement>& PoseWatchElement : Elements)
	{
		PoseWatchElement->SetIsVisible(bInIsVisible);
	}

	bIsVisible = bInIsVisible;

	if (Parent.IsValid())
	{
		Parent->UpdateVisibility();
	}
}

void UPoseWatch::SetColor(const FColor& InColor)
{
	Color_DEPRECATED = InColor;

	for (const TObjectPtr<UPoseWatchElement>& PoseWatchElement : Elements)
	{
		PoseWatchElement->SetColor(InColor);
	}
}

void UPoseWatch::SetShouldDeleteOnDeselect(const bool bInDeleteOnDeselection)
{
	bDeleteOnDeselection = bInDeleteOnDeselection;
}

void UPoseWatch::ToggleIsVisible()
{
	SetIsVisible(!bIsVisible);
}

bool UPoseWatch::IsIn(const UPoseWatchFolder* InFolder) const
{
	return Parent.Get() == InFolder;
}

bool UPoseWatch::IsAssignedFolder() const
{
	return Parent != nullptr;
}

bool UPoseWatch::ValidateLabelRename(const FText& InLabel, FText& OutErrorMessage)
{
	FText UseLabel = FText::TrimPrecedingAndTrailing(InLabel);
	if (UseLabel.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("PoseWatchNameEmpty", "A pose watch must have a label");
		return false;
	}

	if (!IsLabelUniqueInParent(UseLabel, Parent.Get()))
	{
		OutErrorMessage = LOCTEXT("PoseWatchNameTaken", "A pose watch already has this name at this level");
		return false;
	}
	return true;
}

bool UPoseWatch::IsLabelUniqueInParent(const FText& InLabel, UPoseWatchFolder* InFolder) const
{
	for (UPoseWatch* SomePoseWatch : PoseWatchUtil::GetChildrenPoseWatchOf(InFolder, GetAnimBlueprint()))
	{
		if (SomePoseWatch->GetLabel().ToString().Equals(InLabel.ToString()))
		{
			if (SomePoseWatch != this)
			{
				return false;
			}
		}
	}
	return true;
}

void UPoseWatch::SetUniqueDefaultLabel()
{
	Label = GetDefaultLabel();
	Label = FindUniqueNameInParent(Parent.Get());
}

void UPoseWatch::UpdateVisibility()
{
	bIsVisible = false;

	// Set visible if any child component is visible.
	for (const TObjectPtr<UPoseWatchElement>& PoseWatchElement : Elements)
	{
		bIsVisible |= PoseWatchElement->GetIsVisible();
	}
}

UAnimBlueprint* UPoseWatch::GetAnimBlueprint() const
{
	return  CastChecked<UAnimBlueprint>(GetOuter());
}

FText UPoseWatch::FindUniqueNameInParent(UPoseWatchFolder* InParent) const
{
	return PoseWatchUtil::FindUniqueNameInParent(InParent, this);
}

TObjectPtr<UPoseWatchElement> UPoseWatch::AddElement(const FText InLabel, const FName InIconName)
{
	return AddElement<UPoseWatchElement>(InLabel, InIconName);
}

TObjectPtr<UPoseWatchElement> UPoseWatch::FindElement(const FText InLabel)
{
	TObjectPtr<UPoseWatchElement> Element(nullptr);

	if (TObjectPtr<UPoseWatchElement>* FoundElement = GetElements().FindByPredicate([InLabel](const TObjectPtr<UPoseWatchElement>& Element) { return Element->GetLabel().EqualTo(InLabel); }))
	{
		Element = *FoundElement;
	}

	return Element;
}

TObjectPtr<UPoseWatchElement> UPoseWatch::FindOrAddElement(const FText InLabel, const FName InIconName)
{
	TObjectPtr<UPoseWatchElement> Element = FindElement(InLabel);

	if (!Element)
	{
		Element = AddElement(InLabel, InIconName);
	}
	else
	{
		Element->SetIconName(InIconName);
	}

	return Element;
}


#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UPoseWatch::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PoseWatchMigrateSkeletonDrawParametersToPoseElement)
	{
		// Transfer pose watch skeleton draw parameters to a new pose watch pose element.
		TObjectPtr<UPoseWatchPoseElement> NewPoseWatchElement = NewObject<UPoseWatchPoseElement>(this);
		NewPoseWatchElement->SetParent(this);
		NewPoseWatchElement->SetColor(Color_DEPRECATED);
		NewPoseWatchElement->ViewportMask = ViewportMask_DEPRECATED;
		NewPoseWatchElement->bInvertViewportMask = bInvertViewportMask_DEPRECATED;
		NewPoseWatchElement->BlendScaleThreshold = BlendScaleThreshold_DEPRECATED;
		NewPoseWatchElement->ViewportOffset = ViewportOffset_DEPRECATED;
		Elements.Add(NewPoseWatchElement);
	}

	// UE-162694 Remove pose watch elements that were previously incorrectly serialized
	Elements.RemoveAll([](const TObjectPtr<UPoseWatchElement>& Element)
	{
		return Element == nullptr;
	});
}
#endif // WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE
