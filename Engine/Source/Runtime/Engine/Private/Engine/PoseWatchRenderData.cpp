// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/PoseWatchRenderData.h"
#include "Animation/AttributesRuntime.h"

#if WITH_EDITORONLY_DATA

#include "Animation/BlendProfile.h"
#include "Engine/PoseWatch.h"

FAnimNodePoseWatch::FAnimNodePoseWatch()
{
	Object = nullptr;
	PoseWatch = nullptr;
	PoseWatchPoseElement = nullptr;
	NodeID = INDEX_NONE;
	WorldTransform = FTransform::Identity;
	BoneColor = FLinearColor::White;
	ViewportOffset = FVector::ZeroVector;
}

void FAnimNodePoseWatch::SetCurves(const FBlendedCurve& InCurves)
{
	Curves.CopyFrom(InCurves);
}

void FAnimNodePoseWatch::SetAttributes(UE::Anim::FMeshAttributeContainer& InAttributes)
{
	MeshAttributes.MoveFrom(InAttributes);
}

void FAnimNodePoseWatch::SetWorldTransform(const FTransform& InWorldTransform)
{
	WorldTransform = InWorldTransform;
}

void FAnimNodePoseWatch::CopyPoseWatchData(const FReferenceSkeleton& RefSkeleton)
{
	BoneColor = FLinearColor(PoseWatchPoseElement->GetColor());
	ViewportOffset = PoseWatchPoseElement->ViewportOffset;
	ParentIndices.Empty();
	ViewportMaskAllowedList.Empty();

	ParentIndices.AddUninitialized(RefSkeleton.GetNum());
	for (uint16 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
	{
		ParentIndices[BoneIndex] = RefSkeleton.GetParentIndex(BoneIndex);
	}

	const UBlendProfile* Mask = PoseWatchPoseElement->ViewportMask;
	for (const FBoneIndexType& BoneIndex : RequiredBones)
	{
		if (RefSkeleton.IsValidIndex(BoneIndex))
		{
			if (Mask)
			{
				const FName& BoneName = RefSkeleton.GetBoneName(BoneIndex);
				const bool bBoneBlendScaleMeetsThreshold = Mask->GetBoneBlendScale(BoneName) > PoseWatchPoseElement->BlendScaleThreshold;
				const bool bBoneRequired = (!PoseWatchPoseElement->bInvertViewportMask && bBoneBlendScaleMeetsThreshold) || (PoseWatchPoseElement->bInvertViewportMask && !bBoneBlendScaleMeetsThreshold);
				if (!bBoneRequired)
				{
					continue;
				}
			}

			ViewportMaskAllowedList.Add(BoneIndex);
		}
	}
}

const TArray<FBoneIndexType>& FAnimNodePoseWatch::GetRequiredBones() const
{
	return RequiredBones;
}

const TArray<FTransform>& FAnimNodePoseWatch::GetBoneTransforms() const
{
	return BoneTransforms;
}

const FBlendedHeapCurve& FAnimNodePoseWatch::GetCurves() const
{
	return Curves;
}

const UE::Anim::FMeshAttributeContainer& FAnimNodePoseWatch::GetAttributes() const
{
	return MeshAttributes;
}

const FTransform& FAnimNodePoseWatch::GetWorldTransform() const
{
	return WorldTransform;
}

bool FAnimNodePoseWatch::IsValid() const
{
	return Object != nullptr && PoseWatch != nullptr;
}

FLinearColor FAnimNodePoseWatch::GetBoneColor() const
{ 
	return BoneColor;
}

FVector FAnimNodePoseWatch::GetViewportOffset() const
{ 
	return ViewportOffset;
}

const TArray<int32>& FAnimNodePoseWatch::GetViewportAllowList() const
{ 
	return ViewportMaskAllowedList;
}

const TArray<int32>& FAnimNodePoseWatch::GetParentIndices() const
{ 
	return ParentIndices; 
}

#endif // WITH_EDITORONLY_DATA