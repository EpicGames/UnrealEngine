// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ReplicatedState.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicatedState)

// compile switch for disabling quantization of replicated movement, meant for testing.
#ifndef REP_MOVEMENT_DISABLE_QUANTIZATION
#define REP_MOVEMENT_DISABLE_QUANTIZATION 0
#endif

/** If true, origin rebasing is enabled in multiplayer games, meaning that servers and clients can have different local world origins. */
int32 FRepMovement::EnableMultiplayerWorldOriginRebasing = 0;

/** Console variable ref to enable multiplayer world origin rebasing. */
FAutoConsoleVariableRef CVarEnableMultiplayerWorldOriginRebasing(
	TEXT("p.EnableMultiplayerWorldOriginRebasing"),
	FRepMovement::EnableMultiplayerWorldOriginRebasing,
	TEXT("Enable world origin rebasing for multiplayer, meaning that servers and clients can have different world origin locations."),
	ECVF_ReadOnly);

FRepMovement::FRepMovement()
	: LinearVelocity(ForceInit)
	, AngularVelocity(ForceInit)
	, Location(ForceInit)
	, Rotation(ForceInit)
	, Acceleration(ForceInit)
	, bSimulatedPhysicSleep(false)
	, bRepPhysics(false)
	, bRepAcceleration(false)
	, ServerFrame(0)
	, LocationQuantizationLevel(EVectorQuantization::RoundWholeNumber)
	, VelocityQuantizationLevel(EVectorQuantization::RoundWholeNumber)
	, RotationQuantizationLevel(ERotatorQuantization::ByteComponents)
{
}

bool FRepMovement::SerializeQuantizedVector(FArchive& Ar, FVector& Vector, EVectorQuantization QuantizationLevel)
{
	// Since FRepMovement used to use FVector_NetQuantize100, we're allowing enough bits per component
	// regardless of the quantization level so that we can still support at least the same maximum magnitude
	// (2^30 / 100, or ~10 million).
	// This uses no inherent extra bandwidth since we're still using the same number of bits to store the
	// bits-per-component value. Of course, larger magnitudes will still use more bandwidth,
	// as has always been the case.
	switch (QuantizationLevel)
	{
	case EVectorQuantization::RoundTwoDecimals:
	{
		return SerializePackedVector<100, 30>(Vector, Ar);
	}

	case EVectorQuantization::RoundOneDecimal:
	{
		return SerializePackedVector<10, 27>(Vector, Ar);
	}

	default:
	{
		return SerializePackedVector<1, 24>(Vector, Ar);
	}
	}
}

bool FRepMovement::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	// pack bitfield with flags
	const bool bServerFrameAndHandleSupported = Ar.EngineNetVer() >= FEngineNetworkCustomVersion::RepMoveServerFrameAndHandle && Ar.EngineNetVer() != FEngineNetworkCustomVersion::Ver21AndViewPitchOnly_DONOTUSE;
	uint8 Flags = (bSimulatedPhysicSleep << 0) | (bRepPhysics << 1) | ((ServerFrame > 0) << 2) | ((ServerPhysicsHandle != INDEX_NONE) << 3);
	Ar.SerializeBits(&Flags, bServerFrameAndHandleSupported ? 4 : 2);
	bSimulatedPhysicSleep = (Flags & (1 << 0)) ? 1 : 0;
	bRepPhysics = (Flags & (1 << 1)) ? 1 : 0;
	const bool bRepServerFrame = (Flags & (1 << 2) && bServerFrameAndHandleSupported) ? 1 : 0;
	const bool bRepServerHandle = (Flags & (1 << 3) && bServerFrameAndHandleSupported) ? 1 : 0;

	bOutSuccess = true;

#if REP_MOVEMENT_DISABLE_QUANTIZATION
	Ar << Location;
	Ar << LinearVelocity;
	Ar << Rotation;
	if (bRepPhysics)
	{
		Ar << AngularVelocity;
	}
#else

	// update location, rotation, linear velocity
	bOutSuccess &= SerializeQuantizedVector(Ar, Location, LocationQuantizationLevel);

	switch (RotationQuantizationLevel)
	{
	case ERotatorQuantization::ByteComponents:
	{
		Rotation.SerializeCompressed(Ar);
		break;
	}

	case ERotatorQuantization::ShortComponents:
	{
		Rotation.SerializeCompressedShort(Ar);
		break;
	}
	}

	bOutSuccess &= SerializeQuantizedVector(Ar, LinearVelocity, VelocityQuantizationLevel);

	// update angular velocity if required
	if (bRepPhysics)
	{
		bOutSuccess &= SerializeQuantizedVector(Ar, AngularVelocity, VelocityQuantizationLevel);
	}
#endif

	if (bRepServerFrame)
	{
		uint32 uServerFrame = (uint32)ServerFrame;
		Ar.SerializeIntPacked(uServerFrame);
		ServerFrame = (int32)uServerFrame;
	}

	if (bRepServerHandle)
	{
		uint32 uServerPhysicsHandle = (uint32)ServerPhysicsHandle;
		Ar.SerializeIntPacked(uServerPhysicsHandle);
		ServerPhysicsHandle = (int32)uServerPhysicsHandle;
	}

	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::RepMoveOptionalAcceleration)
	{
		uint8 AccelFlags = (bRepAcceleration << 0);
		Ar.SerializeBits(&AccelFlags, 1);
		bRepAcceleration = (AccelFlags & (1 << 0)) ? 1 : 0;

		if (bRepAcceleration)
		{
#if REP_MOVEMENT_DISABLE_QUANTIZATION
			Ar << Acceleration;
#else
			// Note that we're using the same quantization as Velocity, since the units are commonly on the same order
			bOutSuccess &= SerializeQuantizedVector(Ar, Acceleration, VelocityQuantizationLevel);
#endif
		}
	}
	else if (Ar.IsLoading())
	{
		bRepAcceleration = false;
	}

	return true;
}

/** Rebase zero-origin position onto local world origin value. */
FVector FRepMovement::RebaseOntoLocalOrigin(const FVector& Location, const FIntVector& LocalOrigin)
{
	if (EnableMultiplayerWorldOriginRebasing <= 0 || LocalOrigin == FIntVector::ZeroValue)
	{
		return Location;
	}

	return FVector(Location.X - LocalOrigin.X, Location.Y - LocalOrigin.Y, Location.Z - LocalOrigin.Z);
}

/** Rebase local-origin position onto zero world origin value. */
FVector FRepMovement::RebaseOntoZeroOrigin(const FVector& Location, const FIntVector& LocalOrigin)
{
	if (EnableMultiplayerWorldOriginRebasing <= 0 || LocalOrigin == FIntVector::ZeroValue)
	{
		return Location;
	}

	return FVector(Location.X + LocalOrigin.X, Location.Y + LocalOrigin.Y, Location.Z + LocalOrigin.Z);
}

/** Rebase zero-origin position onto local world origin value based on an actor's world. */
FVector FRepMovement::RebaseOntoLocalOrigin(const FVector& Location, const AActor* const WorldContextActor)
{
	if (WorldContextActor == nullptr || EnableMultiplayerWorldOriginRebasing <= 0)
	{
		return Location;
	}

	return RebaseOntoLocalOrigin(Location, WorldContextActor->GetWorld()->OriginLocation);
}

/** Rebase local-origin position onto zero world origin value based on an actor's world.*/
FVector FRepMovement::RebaseOntoZeroOrigin(const FVector& Location, const AActor* const WorldContextActor)
{
	if (WorldContextActor == nullptr || EnableMultiplayerWorldOriginRebasing <= 0)
	{
		return Location;
	}

	return RebaseOntoZeroOrigin(Location, WorldContextActor->GetWorld()->OriginLocation);
}

/// @cond DOXYGEN_WARNINGS

/** Rebase zero-origin position onto local world origin value based on an actor component's world. */
FVector FRepMovement::RebaseOntoLocalOrigin(const FVector& Location, const UActorComponent* const WorldContextActorComponent)
{
	if (WorldContextActorComponent == nullptr || EnableMultiplayerWorldOriginRebasing <= 0)
	{
		return Location;
	}

	return RebaseOntoLocalOrigin(Location, WorldContextActorComponent->GetWorld()->OriginLocation);
}

/** Rebase local-origin position onto zero world origin value based on an actor component's world.*/
FVector FRepMovement::RebaseOntoZeroOrigin(const FVector& Location, const UActorComponent* const WorldContextActorComponent)
{
	if (WorldContextActorComponent == nullptr || EnableMultiplayerWorldOriginRebasing <= 0)
	{
		return Location;
	}

	return RebaseOntoZeroOrigin(Location, WorldContextActorComponent->GetWorld()->OriginLocation);
}
