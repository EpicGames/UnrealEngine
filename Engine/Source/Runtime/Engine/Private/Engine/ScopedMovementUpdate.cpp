// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ScopedMovementUpdate.h"
#include "Components/SkeletalMeshComponent.h"

DEFINE_LOG_CATEGORY_STATIC(LogScopedMovementUpdate, Log, All);

static uint32 s_ScopedWarningCount = 0;

FScopedMovementUpdate::FScopedMovementUpdate( class USceneComponent* Component, EScopedUpdate::Type ScopeBehavior, bool bRequireOverlapsEventFlagToQueueOverlaps )
: Owner(Component)
, OuterDeferredScope(nullptr)
, CurrentOverlapState(EOverlapState::eUseParent)
, TeleportType(ETeleportType::None)
, FinalOverlapCandidatesIndex(INDEX_NONE)
, bDeferUpdates(ScopeBehavior == EScopedUpdate::DeferredUpdates)
, bHasMoved(false)
, bRequireOverlapsEventFlag(bRequireOverlapsEventFlagToQueueOverlaps)
, bIsGroupDeferedUpdate(ScopeBehavior == EScopedUpdate::DeferredGroupUpdates)
{
	if (IsValid(Component))
	{
		OuterDeferredScope = Component->GetCurrentScopedMovement();
		InitialTransform = Component->GetComponentToWorld();
		InitialRelativeLocation = Component->GetRelativeLocation();
		InitialRelativeRotation = Component->GetRelativeRotation();
		InitialRelativeScale = Component->GetRelativeScale3D();

		if (ScopeBehavior == EScopedUpdate::ImmediateUpdates)
		{
			// We only allow ScopeUpdateImmediately if there is no outer scope, or if the outer scope is also ScopeUpdateImmediately.
			if (OuterDeferredScope && OuterDeferredScope->bDeferUpdates)
			{
				if (s_ScopedWarningCount < 100 || (GFrameCounter & 31) == 0)
				{
					s_ScopedWarningCount++;
					UE_LOGF(LogScopedMovementUpdate, Error, "FScopedMovementUpdate attempting to use immediate updates within deferred scope, will use deferred updates instead.");
				}

				bDeferUpdates = true;
			}
		}

		// Do not allow for GROUP deferred updates if there is already an outer scope 
		// which is not part of the group.
		
		// Safe gaurd against other possible input scopes so that you can only use grouped updates on their own, not with others
		if (bIsGroupDeferedUpdate && OuterDeferredScope && !OuterDeferredScope->bIsGroupDeferedUpdate)
		{
			UE_LOGF(LogScopedMovementUpdate, Warning, "FScopedMovementUpdate attempting to use group updates within an ungrouped scope, will use deferred updates instead.");
			bIsGroupDeferedUpdate = false;

			// Fallback to just using normal deferred updates 
			bDeferUpdates = true;
		}

		// Probably better to have a "Begin Grouped movement update" instead to keep them separate. 
		if (bDeferUpdates || bIsGroupDeferedUpdate)
		{
			Component->BeginScopedMovementUpdate(*this);
		}
	}
	else
	{
		Owner = nullptr;
		// Having no owner should be a no-op on all moves, but if someone requests deferred updates on a null component
		// we should not return true for IsDeferringUpdates() so force it off here.
		bDeferUpdates = false;
		bIsGroupDeferedUpdate = false;
	}
}

FScopedMovementUpdate::~FScopedMovementUpdate()
{
	if ((bDeferUpdates || bIsGroupDeferedUpdate) && IsValid(Owner))
	{
		Owner->EndScopedMovementUpdate(*this);
	}
	Owner = nullptr;
}

bool FScopedMovementUpdate::IsTransformDirty() const
{
	if (Owner)
	{
		return !InitialTransform.Equals(Owner->GetComponentToWorld(), UE_SMALL_NUMBER);
	}

	return false;
}

void FScopedMovementUpdate::RevertMove()
{
	USceneComponent* Component = Owner;
	if (IsValid(Component))
	{
		FinalOverlapCandidatesIndex = INDEX_NONE;
		PendingOverlaps.Reset();
		BlockingHits.Reset();
		
		if (IsTransformDirty())
		{
			// Teleport to start
			Component->ComponentToWorld = InitialTransform;
			Component->RelativeLocation = InitialRelativeLocation;
			Component->RelativeRotation = InitialRelativeRotation;
			Component->RelativeScale3D = InitialRelativeScale;
			Component->RelativeRotationCache.RotatorToQuat(InitialRelativeRotation);
			Component->WorldRotationCache.RotatorToQuat(InitialTransform.Rotator());
			
			if (!IsDeferringUpdates())
			{
				Component->PropagateTransformUpdate(true);
				Component->UpdateOverlaps();
			}
		}
	}
	bHasMoved = false;
	CurrentOverlapState = EOverlapState::eUseParent;
	TeleportType = ETeleportType::None;
}

void FScopedMovementUpdate::AppendOverlapsAfterMove(const TOverlapArrayView& NewPendingOverlaps, bool bSweep, bool bIncludesOverlapsAtEnd)
{
	bHasMoved = true;
	const bool bWasForcing = (CurrentOverlapState == EOverlapState::eForceUpdate);

	if (bIncludesOverlapsAtEnd)
	{
		CurrentOverlapState = EOverlapState::eIncludesOverlaps;
		if (NewPendingOverlaps.Num())
		{
			FinalOverlapCandidatesIndex = PendingOverlaps.Num();
			PendingOverlaps.Append(NewPendingOverlaps.GetData(), NewPendingOverlaps.Num());
		}
		else
		{
			// No new pending overlaps means we're not overlapping anything at the end location.
			FinalOverlapCandidatesIndex = INDEX_NONE;
		}
	}
	else
	{
		// We don't know about the final overlaps in the case of a teleport.
		CurrentOverlapState = EOverlapState::eUnknown;
		FinalOverlapCandidatesIndex = INDEX_NONE;
		PendingOverlaps.Append(NewPendingOverlaps.GetData(), NewPendingOverlaps.Num());
	}

	if (bWasForcing)
	{
		CurrentOverlapState = EOverlapState::eForceUpdate;
	}
}

void FScopedMovementUpdate::OnInnerScopeComplete(const FScopedMovementUpdate& InnerScope)
{
	if (IsValid(Owner))
	{
		checkSlow(IsDeferringUpdates());
		checkSlow(InnerScope.IsDeferringUpdates());
		checkSlow(InnerScope.OuterDeferredScope == this);

		// Combine with the next item on the stack.
		if (InnerScope.HasMoved(EHasMovedTransformOption::eTestTransform))
		{
			bHasMoved = true;
			
			if (InnerScope.CurrentOverlapState == EOverlapState::eUseParent)
			{
				// Unchanged, use our own
			}
			else
			{
				// Bubble up from inner scope.
				CurrentOverlapState = InnerScope.CurrentOverlapState;
				if (InnerScope.FinalOverlapCandidatesIndex == INDEX_NONE)
				{
					FinalOverlapCandidatesIndex = INDEX_NONE;
				}
				else
				{
					checkSlow(InnerScope.GetPendingOverlaps().Num() > 0);
					FinalOverlapCandidatesIndex = PendingOverlaps.Num() + InnerScope.FinalOverlapCandidatesIndex;
				}
				PendingOverlaps.Append(InnerScope.GetPendingOverlaps());
				checkSlow(FinalOverlapCandidatesIndex < PendingOverlaps.Num());
			}

			if (InnerScope.TeleportType > TeleportType)
			{
				SetHasTeleported(InnerScope.TeleportType);
			}
		}
		else
		{
			// Don't want to invalidate a parent scope when nothing changed in the child.
			checkSlow(InnerScope.CurrentOverlapState == EOverlapState::eUseParent);
		}

		BlockingHits.Append(InnerScope.GetPendingBlockingHits());
	}	
}

bool FScopedMovementUpdate::SetWorldLocationAndRotation(FVector NewLocation, const FQuat& NewQuat, bool bNoPhysics, ETeleportType Teleport)
{
	if (Owner)
	{
		return Owner->InternalSetWorldLocationAndRotation(NewLocation, NewQuat, bNoPhysics, Teleport);
	}
	return false;
}

TOptional<TOverlapArrayView> FScopedMovementUpdate::GetOverlapsAtEnd(class UPrimitiveComponent& PrimComponent, TInlineOverlapInfoArray& OutEndOverlaps, bool bTransformChanged) const
{
	TOptional<TOverlapArrayView> Result;
	switch (CurrentOverlapState)
	{
	case FScopedMovementUpdate::EOverlapState::eUseParent:
	{
		// Only rotation could have possibly changed
		if (bTransformChanged && PrimComponent.AreSymmetricRotations(InitialTransform.GetRotation(), PrimComponent.GetComponentQuat(), PrimComponent.GetComponentScale()))
		{
			if (PrimComponent.ConvertRotationOverlapsToCurrentOverlaps(OutEndOverlaps, PrimComponent.GetOverlapInfos()))
			{
				Result = OutEndOverlaps;
			}
		}
		else
		{
			// Use current overlaps (unchanged)
			Result = PrimComponent.GetOverlapInfos();
		}
		break;
	}
	case FScopedMovementUpdate::EOverlapState::eUnknown:
	case FScopedMovementUpdate::EOverlapState::eForceUpdate:
	{
		break;
	}
	case FScopedMovementUpdate::EOverlapState::eIncludesOverlaps:
	{
		if (FinalOverlapCandidatesIndex == INDEX_NONE)
		{
			// Overlapping nothing
			Result = OutEndOverlaps;
		}
		else
		{
			// Fill in EndOverlaps with overlaps valid at the end location.
			const bool bMatchingScale = FTransform::AreScale3DsEqual(InitialTransform, PrimComponent.GetComponentTransform());
			if (bMatchingScale)
			{
				const bool bHasEndOverlaps = PrimComponent.ConvertSweptOverlapsToCurrentOverlaps(
					OutEndOverlaps, PendingOverlaps, FinalOverlapCandidatesIndex,
					PrimComponent.GetComponentLocation(), PrimComponent.GetComponentQuat());

				if (bHasEndOverlaps)
				{
					Result = OutEndOverlaps;
				}
			}
		}
		break;
	}
	default:
	{
		checkf(false, TEXT("Unknown FScopedMovementUpdate::EOverlapState value"));
		break;
	}
	}

	return Result;
}
