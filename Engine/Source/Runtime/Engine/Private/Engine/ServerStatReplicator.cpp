// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ServerStatReplicator.h"
#include "Async/TaskGraphInterfaces.h"
#include "Net/UnrealNetwork.h"
#include "EngineStats.h"
#include "Engine/NetDriver.h"
#include "Stats/StatsTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ServerStatReplicator)

AServerStatReplicator::AServerStatReplicator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUpdateStatNet = true;
	bOverwriteClientStats = true;

	// Only replicate to the person that spawned this actor to save on bandwidth
	bReplicates = true;
	bOnlyRelevantToOwner = true;
	NetPriority = 1.f;

#if STATS
	// Tick while alive so we can copy the data over
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
#endif
}

void AServerStatReplicator::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	// No need to replicate any inherited properties
	DISABLE_ALL_CLASS_REPLICATED_PROPERTIES(Super, EFieldIteratorFlags::IncludeSuper);

	DOREPLIFETIME(AServerStatReplicator, Channels);
	DOREPLIFETIME(AServerStatReplicator, InRate);
	DOREPLIFETIME(AServerStatReplicator, OutRate);
	DOREPLIFETIME(AServerStatReplicator, MaxPacketOverhead);
	DOREPLIFETIME(AServerStatReplicator, InRateClientMax);
	DOREPLIFETIME(AServerStatReplicator, InRateClientMin);
	DOREPLIFETIME(AServerStatReplicator, InRateClientAvg);
	DOREPLIFETIME(AServerStatReplicator, InPacketsClientMax);
	DOREPLIFETIME(AServerStatReplicator, InPacketsClientMin);
	DOREPLIFETIME(AServerStatReplicator, InPacketsClientAvg);
	DOREPLIFETIME(AServerStatReplicator, OutRateClientMax);
	DOREPLIFETIME(AServerStatReplicator, OutRateClientMin);
	DOREPLIFETIME(AServerStatReplicator, OutRateClientAvg);
	DOREPLIFETIME(AServerStatReplicator, OutPacketsClientMax);
	DOREPLIFETIME(AServerStatReplicator, OutPacketsClientMin);
	DOREPLIFETIME(AServerStatReplicator, OutPacketsClientAvg);
	DOREPLIFETIME(AServerStatReplicator, NetNumClients);
	DOREPLIFETIME(AServerStatReplicator, InPackets);
	DOREPLIFETIME(AServerStatReplicator, OutPackets);
	DOREPLIFETIME(AServerStatReplicator, InBunches);
	DOREPLIFETIME(AServerStatReplicator, OutBunches);
	DOREPLIFETIME(AServerStatReplicator, OutLoss);
	DOREPLIFETIME(AServerStatReplicator, InLoss);
	DOREPLIFETIME(AServerStatReplicator, VoiceBytesSent);
	DOREPLIFETIME(AServerStatReplicator, VoiceBytesRecv);
	DOREPLIFETIME(AServerStatReplicator, VoicePacketsSent);
	DOREPLIFETIME(AServerStatReplicator, VoicePacketsRecv);
	DOREPLIFETIME(AServerStatReplicator, PercentInVoice);
	DOREPLIFETIME(AServerStatReplicator, PercentOutVoice);
	DOREPLIFETIME(AServerStatReplicator, NumActorChannels);
	DOREPLIFETIME(AServerStatReplicator, NumConsideredActors);
	DOREPLIFETIME(AServerStatReplicator, PrioritizedActors);
	DOREPLIFETIME(AServerStatReplicator, NumRelevantActors);
	DOREPLIFETIME(AServerStatReplicator, NumRelevantDeletedActors);
	DOREPLIFETIME(AServerStatReplicator, NumReplicatedActorAttempts);
	DOREPLIFETIME(AServerStatReplicator, NumReplicatedActors);
	DOREPLIFETIME(AServerStatReplicator, NumActors);
	DOREPLIFETIME(AServerStatReplicator, NumNetActors);
	DOREPLIFETIME(AServerStatReplicator, NumDormantActors);
	DOREPLIFETIME(AServerStatReplicator, NumInitiallyDormantActors);
	DOREPLIFETIME(AServerStatReplicator, NumNetGUIDsAckd);
	DOREPLIFETIME(AServerStatReplicator, NumNetGUIDsPending);
	DOREPLIFETIME(AServerStatReplicator, NumNetGUIDsUnAckd);
	DOREPLIFETIME(AServerStatReplicator, ObjPathBytes);
	DOREPLIFETIME(AServerStatReplicator, NetGUIDOutRate);
	DOREPLIFETIME(AServerStatReplicator, NetGUIDInRate);
	DOREPLIFETIME(AServerStatReplicator, NetSaturated);
}

void AServerStatReplicator::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	if (!bUpdateStatNet)
	{
		return;
	}

#if STATS
	// Copy the net status values over
	if (bOverwriteClientStats)
	{
		SET_DWORD_STAT(STAT_Channels, Channels);
		SET_DWORD_STAT(STAT_MaxPacketOverhead, MaxPacketOverhead);
		SET_DWORD_STAT(STAT_OutLoss, OutLoss);
		SET_DWORD_STAT(STAT_InLoss, InLoss);
		SET_DWORD_STAT(STAT_InRate, InRate);
		SET_DWORD_STAT(STAT_OutRate, OutRate);
		SET_DWORD_STAT(STAT_InPackets, InPackets);
		SET_DWORD_STAT(STAT_OutPackets, OutPackets);
		SET_DWORD_STAT(STAT_InBunches, InBunches);
		SET_DWORD_STAT(STAT_OutBunches, OutBunches);
		SET_DWORD_STAT(STAT_NetGUIDInRate, NetGUIDInRate);
		SET_DWORD_STAT(STAT_NetGUIDOutRate, NetGUIDOutRate);
		SET_DWORD_STAT(STAT_VoicePacketsSent, VoicePacketsSent);
		SET_DWORD_STAT(STAT_VoicePacketsRecv, VoicePacketsRecv);
		SET_DWORD_STAT(STAT_VoiceBytesSent, VoiceBytesSent);
		SET_DWORD_STAT(STAT_VoiceBytesRecv, VoiceBytesRecv);
		SET_DWORD_STAT(STAT_PercentInVoice, PercentInVoice);
		SET_DWORD_STAT(STAT_PercentOutVoice, PercentOutVoice);
		SET_DWORD_STAT(STAT_NumActorChannels, NumActorChannels);
		SET_DWORD_STAT(STAT_NumDormantActors, NumDormantActors);
		SET_DWORD_STAT(STAT_NumActors, NumActors);
		SET_DWORD_STAT(STAT_NumNetActors, NumNetActors);
		SET_DWORD_STAT(STAT_NumNetGUIDsAckd, NumNetGUIDsAckd);
		SET_DWORD_STAT(STAT_NumNetGUIDsPending, NumNetGUIDsPending);
		SET_DWORD_STAT(STAT_NumNetGUIDsUnAckd, NumNetGUIDsUnAckd);
		SET_DWORD_STAT(STAT_NetSaturated, NetSaturated);
	}
	SET_DWORD_STAT(STAT_InRateClientMax, InRateClientMax);
	SET_DWORD_STAT(STAT_InRateClientMin, InRateClientMin);
	SET_DWORD_STAT(STAT_InRateClientAvg, InRateClientAvg);
	SET_DWORD_STAT(STAT_InPacketsClientMax, InPacketsClientMax);
	SET_DWORD_STAT(STAT_InPacketsClientMin, InPacketsClientMin);
	SET_DWORD_STAT(STAT_InPacketsClientAvg, InPacketsClientAvg);
	SET_DWORD_STAT(STAT_OutRateClientMax, OutRateClientMax);
	SET_DWORD_STAT(STAT_OutRateClientMin, OutRateClientMin);
	SET_DWORD_STAT(STAT_OutRateClientAvg, OutRateClientAvg);
	SET_DWORD_STAT(STAT_OutPacketsClientMax, OutPacketsClientMax);
	SET_DWORD_STAT(STAT_OutPacketsClientMin, OutPacketsClientMin);
	SET_DWORD_STAT(STAT_OutPacketsClientAvg, OutPacketsClientAvg);
	SET_DWORD_STAT(STAT_NetNumClients, NetNumClients);
#endif
}

void AServerStatReplicator::BeginPlay()
{
	Super::BeginPlay();
	// This only makes sense on a pure client
	if (!IsNetMode(NM_Client))
	{
		SetActorTickEnabled(false);
		return;
	}
	else
	{
		UNetDriver* NetDriver = GetNetDriver();
		NetDriver->bSkipLocalStats = true;
	}
}

void AServerStatReplicator::Destroyed()
{
	if (IsNetMode(NM_Client))
	{
		UNetDriver* NetDriver = GetNetDriver();
		NetDriver->bSkipLocalStats = false;
	}
	Super::Destroyed();
}


