// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SystemTimeTimecodeProvider.h"

#include "Engine/TimecodeProvider.h"
#include <cmath>

#include UE_INLINE_GENERATED_CPP_BY_NAME(SystemTimeTimecodeProvider)

namespace
{
	static double ComputeTimeCodeOffset()
	{
		const FDateTime DateTime = FDateTime::Now();
		const double HighPerformanceClock = FPlatformTime::Seconds();
		return DateTime.GetTimeOfDay().GetTotalSeconds() - HighPerformanceClock;
	}

	static double HighPerformanceClockDelta = ComputeTimeCodeOffset();
};


USystemTimeTimecodeProvider::USystemTimeTimecodeProvider()
	: FrameRate(60, 1)
	, bGenerateFullFrame(true)
	, bUseHighPerformanceClock(false)
	, State(ETimecodeProviderSynchronizationState::Closed)
{
}


FFrameTime USystemTimeTimecodeProvider::GenerateFrameTimeFromSystemTime(FFrameRate FrameRate)
{
	const FDateTime DateTime = FDateTime::Now();
	const FTimespan Timespan = DateTime.GetTimeOfDay();
	return FrameRate.AsFrameTime(Timespan.GetTotalSeconds());
}


FTimecode USystemTimeTimecodeProvider::GenerateTimecodeFromSystemTime(FFrameRate FrameRate)
{
	const FDateTime DateTime = FDateTime::Now();
	const FTimespan Timespan = DateTime.GetTimeOfDay();
	return FTimecode::FromTimespan(Timespan, FrameRate, false);
}


FFrameTime USystemTimeTimecodeProvider::GenerateFrameTimeFromHighPerformanceClock(FFrameRate FrameRate)
{
	constexpr double SecondsPerDay = 24.0 * 60.0 * 60.0;
	return FrameRate.AsFrameTime(fmod(HighPerformanceClockDelta + FPlatformTime::Seconds(), SecondsPerDay));
}


FTimecode USystemTimeTimecodeProvider::GenerateTimecodeFromHighPerformanceClock(FFrameRate FrameRate)
{
	constexpr double SecondsPerDay = 24.0 * 60.0 * 60.0;
	const FTimespan Timespan = FTimespan::FromSeconds(fmod(HighPerformanceClockDelta + FPlatformTime::Seconds(), SecondsPerDay));
	return FTimecode::FromTimespan(Timespan, FrameRate, false);
}


FQualifiedFrameTime USystemTimeTimecodeProvider::GetQualifiedFrameTime() const
{
	return bGenerateFullFrame ? 
		FQualifiedFrameTime(bUseHighPerformanceClock ? GenerateTimecodeFromHighPerformanceClock(FrameRate) : GenerateTimecodeFromSystemTime(FrameRate), FrameRate)
		: FQualifiedFrameTime(bUseHighPerformanceClock ? GenerateFrameTimeFromHighPerformanceClock(FrameRate) : GenerateFrameTimeFromSystemTime(FrameRate), FrameRate);
}

