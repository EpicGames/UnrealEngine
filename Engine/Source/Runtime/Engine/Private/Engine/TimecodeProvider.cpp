// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/TimecodeProvider.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimecodeProvider)

namespace UE::Time
{
	static FAutoConsoleCommand TimecodeProviderReinitializeCmd(
		TEXT("TimecodeProvider.Reinitialize"),
		TEXT("Reinitialize the current timecode provider"),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				if (GEngine)
				{
					GEngine->ReinitializeTimecodeProvider();
				}
			})
	);
}


FQualifiedFrameTime UTimecodeProvider::GetDelayedQualifiedFrameTime() const
{
	FQualifiedFrameTime NewFrameTime = GetQualifiedFrameTime();
	NewFrameTime.Time -= FFrameTime::FromDecimal(FrameDelay);
	return NewFrameTime;
}


FTimecode UTimecodeProvider::GetTimecode() const
{
	return GetQualifiedFrameTime().ToTimecode();
}


FTimecode UTimecodeProvider::GetDelayedTimecode() const
{
	return GetDelayedQualifiedFrameTime().ToTimecode();
}

