// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ViewportStatsSubsystem.h"
#include "CanvasItem.h"
#include "Engine/World.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"      // GEngine
#include "TimerManager.h"		// FTimerDelegate, FTimerHandle

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewportStatsSubsystem)

DECLARE_LOG_CATEGORY_EXTERN(LogViewportStatsSubsystem, Log, All);
DEFINE_LOG_CATEGORY(LogViewportStatsSubsystem);

///////////////////////////////////////////////////
//FViewportDisplayDelegate
bool FViewportDisplayDelegate::Execute(FText& OutText, FLinearColor& OutColor)
{
	if (FuncDynDelegate.IsBound())
	{
		return FuncDynDelegate.Execute(OutText, OutColor);
	}
	else if (FuncCallback)
	{
		return FuncCallback(OutText, OutColor);
	}

	return false;
}

void FViewportDisplayDelegate::Unbind()
{
	FuncDynDelegate.Unbind();
	FuncCallback = nullptr;
}

///////////////////////////////////////////////////
// UViewportStatsSubsystem
#define LOCTEXT_NAMESPACE "ViewportStatsSubsystem"

void UViewportStatsSubsystem::Deinitialize()
{
	Super::Deinitialize();

	DisplayDelegates.Empty();
	UniqueDisplayMessages.Empty();
}

void UViewportStatsSubsystem::Draw(FViewport* Viewport, FCanvas* Canvas, UCanvas* CanvasObject, float MessageStartY)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UViewportStatsSubsystem::Draw);

	static const float FontSizeY = 20.0f;

	FVector2D MessagePos { 20.f, MessageStartY };

	FCanvasTextItem SmallTextItem(FVector2D(0.0f, 0.0f), FText::GetEmpty(), GEngine->GetSmallFont(), FLinearColor::White);
	SmallTextItem.Scale = FVector2D(1.0f, 1.0f);

	SmallTextItem.EnableShadow(FLinearColor::Black);

	// Display one off messages or timed ones
	for(TSharedPtr<FUniqueDisplayData> Message : UniqueDisplayMessages)
	{
		SmallTextItem.Text = Message->DisplayText;
		SmallTextItem.SetColor(Message->DisplayColor);
		Canvas->DrawItem(SmallTextItem, MessagePos + Message->DisplayOffset);

		MessagePos.Y += FontSizeY;
	}

	// Display any stateful messages that we may have delegate for
	FText OutText = FText::GetEmpty();
	FLinearColor OutCol = FLinearColor::White;

	for(FViewportDisplayDelegate& Handle : DisplayDelegates)
	{
		if (Handle.IsBound() && Handle.Execute(OutText, OutCol))
		{
			SmallTextItem.Text = OutText;
			SmallTextItem.SetColor(OutCol);
			Canvas->DrawItem(SmallTextItem, MessagePos);

			MessagePos.Y += FontSizeY;
		}
	}
}

int32 UViewportStatsSubsystem::AddDisplayDelegate(FViewportDisplayCallback const& Delegate)
{
	return DisplayDelegates.Add(FViewportDisplayDelegate(Delegate));
}

int32 UViewportStatsSubsystem::AddDisplayDelegate(FShouldDisplayFunc&& Callback)
{
	return DisplayDelegates.Add(FViewportDisplayDelegate(MoveTemp(Callback)));
}

void UViewportStatsSubsystem::RemoveDisplayDelegate(const int32 IndexToRemove)
{
	// If this is a valid index than unbind and remove this delegate from the callbacks
	if(IndexToRemove >= 0 && IndexToRemove < DisplayDelegates.Num())
	{
		DisplayDelegates[IndexToRemove].Unbind();
		DisplayDelegates.RemoveAtSwap(IndexToRemove);
	}
}

void UViewportStatsSubsystem::AddTimedDisplay(FText Text, FLinearColor Color, float Duration, const FVector2D& DisplayOffset /* = FVector2D::ZeroVector */)
{
	UWorld* MyWorld = GetWorld();
	
	if(!MyWorld)
	{
		return;
	}

	TSharedPtr<FUniqueDisplayData> Message = MakeShared<FUniqueDisplayData>(Text, Color, DisplayOffset);
	UniqueDisplayMessages.Add(Message);

	// If the user has specified a duration, then remove the message after it
	if(Duration != 0.0f)
	{
		FTimerDelegate TimerDel;
		FTimerHandle TimerHandle;

		auto RemoveAfterSecondsLambda = [](TWeakPtr<FUniqueDisplayData> DisplayItem, TArray<TSharedPtr<FUniqueDisplayData>>* ConditionArray)
		{
			if (DisplayItem.IsValid() && ConditionArray)
			{
				ConditionArray->Remove(DisplayItem.Pin());
			}
		};

		TimerDel.BindLambda(RemoveAfterSecondsLambda, TWeakPtr<FUniqueDisplayData>(Message), &UniqueDisplayMessages);
		MyWorld->GetTimerManager().SetTimer(TimerHandle, TimerDel, Duration, /* bInLoop= */ false);
	}
}

#undef LOCTEXT_NAMESPACE

