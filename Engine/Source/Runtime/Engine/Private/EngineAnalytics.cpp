// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineAnalytics.h"
#include "Misc/App.h"
#include "Stats/Stats.h"
#include "Misc/ConfigCacheIni.h"
#include "Engine/Engine.h"
#include "Misc/EngineBuildSettings.h"
#include "AnalyticsBuildType.h"
#include "IAnalyticsProviderET.h"
#include "GeneralProjectSettings.h"
#include "Misc/EngineVersion.h"
#include "BuildSettings.h"
#include "RHI.h"
#include "RHIStats.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "UObject/Class.h"
#include "Containers/Set.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"

#if WITH_EDITOR
#include "AnalyticsSessionSummaryManager.h"
#include "AnalyticsSessionSummarySender.h"
#include "Analytics/EditorAnalyticsSessionSummary.h"
#include "EditorAnalyticsSession.h" // DEPRECATED: kept around to clean up expired old sessions.
#include "Horde.h"
#endif

bool FEngineAnalytics::bIsInitialized;
TSharedPtr<IAnalyticsProviderET> FEngineAnalytics::Analytics;
TSet<FString> FEngineAnalytics::SessionEpicAccountIds;

#if WITH_EDITOR
static TUniquePtr<FAnalyticsSessionSummaryManager> AnalyticsSessionSummaryManager;
static TUniquePtr<FEditorAnalyticsSessionSummary> EditorAnalyticSessionSummary;
static TSharedPtr<FAnalyticsSessionSummarySender> AnalyticsSessionSummarySender;
FSimpleMulticastDelegate FEngineAnalytics::OnInitializeEngineAnalytics;
FSimpleMulticastDelegate FEngineAnalytics::OnShutdownEngineAnalytics;
#endif

namespace UE::Analytics::Private
{

IEngineAnalyticsConfigOverride* EngineAnalyticsConfigOverride = nullptr;

}

static TSharedPtr<IAnalyticsProviderET> CreateEpicAnalyticsProvider()
{
	FAnalyticsET::Config Config;
	{
		// We always use the "Release" analytics account unless we're running in analytics test mode (usually with
		// a command-line parameter), or we're an internal Epic build
		const EAnalyticsBuildType AnalyticsBuildType = GetAnalyticsBuildType();
		const bool bUseReleaseAccount =
			(AnalyticsBuildType == EAnalyticsBuildType::Development || AnalyticsBuildType == EAnalyticsBuildType::Release) &&
			!FEngineBuildSettings::IsInternalBuild();	// Internal Epic build
		const TCHAR* BuildTypeStr = bUseReleaseAccount ? TEXT("Release") : TEXT("Dev");

		FString UETypeOverride;
		bool bHasOverride = GConfig->GetString(TEXT("Analytics"), TEXT("UE4TypeOverride"), UETypeOverride, GEngineIni);
		const TCHAR* UETypeStr = bHasOverride ? *UETypeOverride : FEngineBuildSettings::IsPerforceBuild() ? TEXT("Perforce") : TEXT("UnrealEngine");

		FString AppID;
		GConfig->GetString(TEXT("Analytics"), TEXT("AppIdOverride"), AppID, GEditorIni);
		Config.APIKeyET = FString::Printf(TEXT("%s.%s.%s"), AppID.IsEmpty() ? TEXT("UEEditor") : *AppID, UETypeStr, BuildTypeStr);
	}
	Config.APIServerET = TEXT("https://datarouter.ol.epicgames.com/");
	Config.AppEnvironment = TEXT("datacollector-binary");
	Config.AppVersionET = FEngineVersion::Current().ToString();

	if (UE::Analytics::Private::EngineAnalyticsConfigOverride)
	{
		UE::Analytics::Private::EngineAnalyticsConfigOverride->ApplyConfiguration(Config);
	}

	// Connect the engine analytics provider (if there is a configuration delegate installed)
	return FAnalyticsET::Get().CreateAnalyticsProvider(Config);
}

IAnalyticsProviderET& FEngineAnalytics::GetProvider()
{
	checkf(bIsInitialized && IsAvailable(), TEXT("FEngineAnalytics::GetProvider called outside of Initialize/Shutdown."));

	return *Analytics.Get();
}

#if WITH_EDITOR
FAnalyticsSessionSummaryManager& FEngineAnalytics::GetSummaryManager()
{
	checkf(bIsInitialized && AnalyticsSessionSummaryManager.IsValid(), TEXT("FEngineAnalytics::GetSessionManager called outside of Initialize/Shutdown."));

	return *AnalyticsSessionSummaryManager.Get();
}
#endif

void FEngineAnalytics::Initialize()
{
	checkf(!bIsInitialized, TEXT("FEngineAnalytics::Initialize called more than once."));

	check(GEngine);

#if WITH_EDITOR
	// this will only be true for builds that have editor support (desktop platforms)
	// The idea here is to only send editor events for actual editor runs, not for things like -game runs of the editor.
	bool bIsEditorRun = GIsEditor && !IsRunningCommandlet();
#else
	bool bIsEditorRun = false;
#endif

#if UE_BUILD_DEBUG
	const bool bShouldInitAnalytics = false;
#else
	// Outside of the editor, the only engine analytics usage is the hardware survey
	const bool bShouldInitAnalytics = bIsEditorRun && GEngine->AreEditorAnalyticsEnabled();
#endif

	if (bShouldInitAnalytics)
	{
		Analytics = CreateEpicAnalyticsProvider();

		if (Analytics.IsValid())
		{
			Analytics->SetUserID(CreateAnalyticsUserId(FPlatformMisc::GetEpicAccountId()));

			if (UE::Analytics::Private::EngineAnalyticsConfigOverride)
			{
				UE::Analytics::Private::EngineAnalyticsConfigOverride->OnInitialized(*Analytics, UE::Analytics::Private::FOnEpicAccountIdChanged::CreateStatic(&FEngineAnalytics::OnEpicAccountIdChanged));
			}

			TArray<FAnalyticsEventAttribute> StartSessionAttributes;
			AppendMachineStats(StartSessionAttributes);

			Analytics->StartSession(MoveTemp(StartSessionAttributes));
			SendMachineInfoForAccount(FPlatformMisc::GetEpicAccountId());
			bIsInitialized = true;
		}

#if WITH_EDITOR
		// Avoid crashing if Analytics did not initialize properly for whatever reason.
		if (Analytics.IsValid() && !AnalyticsSessionSummaryManager)
		{
			// Create the session summary manager for the Editor instance.
			AnalyticsSessionSummaryManager = MakeUnique<FAnalyticsSessionSummaryManager>(
				TEXT("Editor"), // The tag name of the process.
				FApp::GetInstanceId().ToString(EGuidFormats::Digits), // Unique key to link the principal process (Editor) with subsidiary processes (CRC), that key is passed to CRC.
				Analytics->GetUserID(),
				Analytics->GetAppID(),
				Analytics->GetAppVersion(),
				Analytics->GetSessionID());

			// The sender will sends orphans sessions and maybe this session if CRC dies first.
			AnalyticsSessionSummaryManager->SetSender(MakeShared<FAnalyticsSessionSummarySender>(FEngineAnalytics::GetProvider()));

			// Create a property store file with enough pre-reserved capacity to store the analytics data. This reduce risk of running out of disk later.
			constexpr uint32 ReservedFileCapacity = 16 * 1024;
			if (TSharedPtr<IAnalyticsPropertyStore> EditorPropertyStore = AnalyticsSessionSummaryManager->MakeStore(ReservedFileCapacity))
			{
				// Create the object responsible to collect the Editor session properties.
				EditorAnalyticSessionSummary = MakeUnique<FEditorAnalyticsSessionSummary>(EditorPropertyStore, FGenericCrashContext::GetOutOfProcessCrashReporterProcessId());
			}

			OnInitializeEngineAnalytics.Broadcast();
		}
#endif
	}
}

void FEngineAnalytics::Shutdown(bool bIsEngineShutdown)
{
#if WITH_EDITOR
	OnShutdownEngineAnalytics.Broadcast();

	if (EditorAnalyticSessionSummary)
	{
		EditorAnalyticSessionSummary->Shutdown();
		EditorAnalyticSessionSummary.Reset();
	}

	if (AnalyticsSessionSummaryManager)
	{
		bool bDiscard = !bIsEngineShutdown; // User toggled the 'Send Data' off.
		AnalyticsSessionSummaryManager->Shutdown(bDiscard);
		AnalyticsSessionSummaryManager.Reset();
	}
	else
	{
		// The manager cleans any left-over (crash, power outage) on shutdown  when analytics is on but if off, ensure to clean up what could be left from when it was on.
		FAnalyticsSessionSummaryManager::CleanupExpiredFiles();
	}

	// Clean up the outdated sessions created by the deprecated system. If an older compabile Editor is launched, it can still send them up before they get expired.
	CleanupDeprecatedAnalyticSessions(FAnalyticsSessionSummaryManager::GetSessionExpirationAge());
#endif

	bIsInitialized = false;

	ensure(!Analytics.IsValid() || Analytics.IsUnique());
	Analytics.Reset();
}

void FEngineAnalytics::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FEngineAnalytics_Tick);

#if WITH_EDITOR
	if (EditorAnalyticSessionSummary)
	{
		EditorAnalyticSessionSummary->Tick(DeltaTime);
	}

	if (AnalyticsSessionSummaryManager.IsValid())
	{
		AnalyticsSessionSummaryManager->Tick();
	}
#endif
}

void FEngineAnalytics::LowDriveSpaceDetected()
{
#if WITH_EDITOR
	if (EditorAnalyticSessionSummary)
	{
		EditorAnalyticSessionSummary->LowDriveSpaceDetected();
	}
#endif
}

FString FEngineAnalytics::CreateAnalyticsUserId(const FString& EpicAccountId)
{
	return FString::Printf(TEXT("%s|%s|%s"), *FPlatformMisc::GetLoginId(), *EpicAccountId, *FPlatformMisc::GetOperatingSystemId());
}

void FEngineAnalytics::AppendMachineStats(TArray<FAnalyticsEventAttribute>& EventAttributes)
{
	const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
	const FPlatformMemoryStats Stats = FPlatformMemory::GetStats();

	FString OSMajor;
	FString OSMinor;
	FPlatformMisc::GetOSVersions(/*out*/ OSMajor, /*out*/ OSMinor);

	FTextureMemoryStats TextureMemStats;
	RHIGetTextureMemoryStats(TextureMemStats);

	GEngine->CreateStartupAnalyticsAttributes( EventAttributes );

	// Add project info whether we are in editor or game.
	EventAttributes.Emplace(TEXT("ProjectName"), ProjectSettings.ProjectName);
	EventAttributes.Emplace(TEXT("ProjectID"), ProjectSettings.ProjectID);
	EventAttributes.Emplace(TEXT("ProjectDescription"), ProjectSettings.Description);
	EventAttributes.Emplace(TEXT("ProjectVersion"), ProjectSettings.ProjectVersion);
	EventAttributes.Emplace(TEXT("Application.Commandline"), FCommandLine::Get());
	EventAttributes.Emplace(TEXT("Build.Configuration"), LexToString(FApp::GetBuildConfiguration()));
	EventAttributes.Emplace(TEXT("Build.IsInternalBuild"), FEngineBuildSettings::IsInternalBuild());
	EventAttributes.Emplace(TEXT("Build.IsPerforceBuild"), FEngineBuildSettings::IsPerforceBuild());
	EventAttributes.Emplace(TEXT("Build.IsPromotedBuild"), FApp::GetEngineIsPromotedBuild() == 0 ? false : true);
	EventAttributes.Emplace(TEXT("Build.BranchName"), FApp::GetBranchName());
	EventAttributes.Emplace(TEXT("Build.Changelist"), BuildSettings::GetCurrentChangelist());
	EventAttributes.Emplace(TEXT("Config.IsEditor"), GIsEditor);
	EventAttributes.Emplace(TEXT("Config.IsUnattended"), FApp::IsUnattended());
	EventAttributes.Emplace(TEXT("Config.IsBuildMachine"), GIsBuildMachine);
	EventAttributes.Emplace(TEXT("Config.IsRunningCommandlet"), IsRunningCommandlet());
	EventAttributes.Emplace(TEXT("Platform.IsRemoteSession"), FPlatformMisc::IsRemoteSession());
	EventAttributes.Emplace(TEXT("OSMajor"), OSMajor);
	EventAttributes.Emplace(TEXT("OSMinor"), OSMinor);
	EventAttributes.Emplace(TEXT("OSVersion"), FPlatformMisc::GetOSVersion());
	EventAttributes.Emplace(TEXT("Is64BitOS"), true);
	EventAttributes.Emplace(TEXT("GPUVendorID"), GRHIVendorId);
	EventAttributes.Emplace(TEXT("GPUDeviceID"), GRHIDeviceId);
	EventAttributes.Emplace(TEXT("GPUMemory"), static_cast<uint64>(TextureMemStats.DedicatedVideoMemory));
	EventAttributes.Emplace(TEXT("GRHIDeviceRevision"), GRHIDeviceRevision);
	EventAttributes.Emplace(TEXT("GRHIAdapterInternalDriverVersion"), GRHIAdapterInternalDriverVersion);
	EventAttributes.Emplace(TEXT("GRHIAdapterUserDriverVersion"), GRHIAdapterUserDriverVersion);
	EventAttributes.Emplace(TEXT("GRHIName"), FString(GDynamicRHI ? GDynamicRHI->GetName() : TEXT("")));
	EventAttributes.Emplace(TEXT("TotalPhysicalRAM"), static_cast<uint64>(Stats.TotalPhysical));
	EventAttributes.Emplace(TEXT("CPUPhysicalCores"), FPlatformMisc::NumberOfCores());
	EventAttributes.Emplace(TEXT("CPULogicalCores"), FPlatformMisc::NumberOfCoresIncludingHyperthreads());
	EventAttributes.Emplace(TEXT("DesktopGPUAdapter"), FPlatformMisc::GetPrimaryGPUBrand());
	EventAttributes.Emplace(TEXT("RenderingGPUAdapter"), GRHIAdapterName);
	EventAttributes.Emplace(TEXT("CPUVendor"), FPlatformMisc::GetCPUVendor());
	EventAttributes.Emplace(TEXT("CPUBrand"), FPlatformMisc::GetCPUBrand());

	// Send Internationalization setting of the editor
	EventAttributes.Emplace(TEXT("Internationalization.Language"), FInternationalization::Get().GetCurrentLanguage()->GetName());
	EventAttributes.Emplace(TEXT("Internationalization.Locale"), FInternationalization::Get().GetCurrentLocale()->GetName());

#if WITH_EDITOR
	EventAttributes.Emplace(TEXT("Horde.TemplateID"), FHorde::GetTemplateId());
	EventAttributes.Emplace(TEXT("Horde.TemplateName"), FHorde::GetTemplateName());
	EventAttributes.Emplace(TEXT("Horde.JobURL"), FHorde::GetJobURL());
	EventAttributes.Emplace(TEXT("Horde.JobID"), FHorde::GetJobId());
	EventAttributes.Emplace(TEXT("Horde.StepName"), FHorde::GetStepName());
	EventAttributes.Emplace(TEXT("Horde.StepID"), FHorde::GetStepId());
	EventAttributes.Emplace(TEXT("Horde.StepURL"), FHorde::GetStepURL());
	EventAttributes.Emplace(TEXT("Horde.BatchID"), FHorde::GetBatchId());
#endif

#if PLATFORM_MAC
#if PLATFORM_MAC_ARM64
	EventAttributes.Emplace(TEXT("UEBuildArch"), FString("AppleSilicon"));
#else
	EventAttributes.Emplace(TEXT("UEBuildArch"), FString("Intel(Mac)"));
#endif
#endif
}

void FEngineAnalytics::SendMachineInfoForAccount(const FString& EpicAccountId)
{
	// Note: EpicAccountId may be empty when the user has not signed in to the epic games launcher.
	// The intention here is to only send SessionMachineStats once per unique user including when
	// no user has logged in.
	if (Analytics && !SessionEpicAccountIds.Contains(EpicAccountId))
	{
		SessionEpicAccountIds.Add(EpicAccountId);

		TArray<FAnalyticsEventAttribute> EventAttributes;
		AppendMachineStats(EventAttributes);

		// When the user id is changed send a SessionMachineStats event.
		static const FString SZEventName = "SessionMachineStats";
		Analytics->RecordEvent(SZEventName, EventAttributes);
	}
}

void FEngineAnalytics::OnEpicAccountIdChanged(const FString& EpicAccountId)
{
	// For analytics reporting ignore changes to an empty account id when the user logs out.
	if (!EpicAccountId.IsEmpty())
	{
		const FString NewAnalyticsUserId = CreateAnalyticsUserId(EpicAccountId);

		// Update analytics provider user.
		if (Analytics)
		{
			Analytics->SetUserID(NewAnalyticsUserId);
			SendMachineInfoForAccount(EpicAccountId);
		}

#if WITH_EDITOR
		// Update the summary manager and all of the data stores.
		if (AnalyticsSessionSummaryManager)
		{
			AnalyticsSessionSummaryManager->SetUserId(NewAnalyticsUserId);
		}
#endif

		// Update the crash context so the user id will be sent with runtime events to CRCEditor.
		FGenericCrashContext::SetEpicAccountId(EpicAccountId);
	}
}
