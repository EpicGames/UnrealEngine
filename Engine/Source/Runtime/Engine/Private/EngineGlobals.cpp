// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Engine.cpp: Unreal engine package.
=============================================================================*/

#include "EngineGlobals.h"
#include "ComponentReregisterContext.h"
#include "EditorSupportDelegates.h"
#include "MaterialShared.h"
#include "Modules/ModuleManager.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "RendererInterface.h"

#if !UE_BUILD_SHIPPING && PLATFORM_DESKTOP 
#include "ISlateReflectorModule.h"
#include "TranslationEditorModule.h"
#endif

class IRendererModule;

/*-----------------------------------------------------------------------------
	Globals.
-----------------------------------------------------------------------------*/

// Suppress linker warning "warning LNK4221: no public symbols found; archive member will be inaccessible"
int32 EngineLinkerHelper;


#if WITH_EDITOR
/*-----------------------------------------------------------------------------
 FEditorSupportDelegates
 Delegates that are needed for proper editor functionality, but are accessed 
 or triggered in engine code.
 -----------------------------------------------------------------------------*/
/** Called when all viewports need to be redrawn */
FSimpleMulticastDelegate FEditorSupportDelegates::RedrawAllViewports;
/** Called when the editor is about to cleanse an object that *must* be purged (such as when changing the active map or level) */
FEditorSupportDelegates::FPrepareToCleanseEditorObject FEditorSupportDelegates::PrepareToCleanseEditorObject;
/** Called when the editor is cleansing of transient references before a map change event */
FSimpleMulticastDelegate FEditorSupportDelegates::CleanseEditor;
/** Called when the world is modified */
FSimpleMulticastDelegate FEditorSupportDelegates::WorldChange;
/** Sent to force a property window rebuild */
FEditorSupportDelegates::FOnForcePropertyWindowRebuild FEditorSupportDelegates::ForcePropertyWindowRebuild;
/** Sent when events happen that affect how the editors UI looks (mode changes, grid size changes, etc) */
FSimpleMulticastDelegate FEditorSupportDelegates::UpdateUI;
/** Refresh property windows w/o creating/destroying controls */
FSimpleMulticastDelegate FEditorSupportDelegates::RefreshPropertyWindows;
/** Sent before the given windows message is handled in the given viewport */
FEditorSupportDelegates::FOnWindowsMessage FEditorSupportDelegates::PreWindowsMessage;
/** Sent after the given windows message is handled in the given viewport */
FEditorSupportDelegates::FOnWindowsMessage FEditorSupportDelegates::PostWindowsMessage;
/** Sent after the usages flags on a material have changed*/
FEditorSupportDelegates::FOnMaterialUsageFlagsChanged FEditorSupportDelegates::MaterialUsageFlagsChanged;
FEditorSupportDelegates::FOnNumericParameterDefaultChanged FEditorSupportDelegates::NumericParameterDefaultChanged;

#endif // WITH_EDITOR

IRendererModule* CachedRendererModule = NULL;

ENGINE_API IRendererModule& GetRendererModule()
{
	if (!CachedRendererModule)
	{
		CachedRendererModule = &FModuleManager::LoadModuleChecked<IRendererModule>(TEXT("Renderer"));
	}

	return *CachedRendererModule;
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

void ReattachMaterialInstances(const TArray<FString>& Args)
{
	FMaterialUpdateContext MaterialUpdateContext;

	UE_LOGF(LogConsoleResponse, Display, "Reattach.MaterialInstances:");

	// Clear the parents out of combination material instances
	for( TObjectIterator<UMaterialInstanceConstant> MaterialIt; MaterialIt; ++MaterialIt )
	{
		if(Args.Num() == 1)
		{
			UE_LOGF(LogConsoleResponse, Display, "   %ls", *MaterialIt->GetName());

			if(MaterialIt->GetName() == Args[0])
			{
				MaterialUpdateContext.AddMaterialInstance(*MaterialIt);
			}
		}
	}
	UE_LOGF(LogConsoleResponse, Display, "");
}

FAutoConsoleCommand ReattachMaterialInstancesCmd(
	TEXT("Reattach.MaterialInstances"),
	TEXT("Useful for debugging, reattaches all materials. Optional parameter can be a materialinstance name (e.g. DecoStatue_Subsurface0)."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&ReattachMaterialInstances)
	);

void ReattachMaterials(const TArray<FString>& Args)
{
	FMaterialUpdateContext MaterialUpdateContext;

	UE_LOGF(LogConsoleResponse, Display, "Reattach.Materials:");

	// Clear the parents out of combination material
	for( TObjectIterator<UMaterial> MaterialIt; MaterialIt; ++MaterialIt )
	{
		if(Args.Num() == 1)
		{
			UE_LOGF(LogConsoleResponse, Display, "   %ls", *MaterialIt->GetName());

			if(MaterialIt->GetName() == Args[0])
			{
				MaterialUpdateContext.AddMaterial(*MaterialIt);
			}
		}
	}
	UE_LOGF(LogConsoleResponse, Display, "");
}

FAutoConsoleCommand ReattachMaterialsCmd(
	TEXT("Reattach.Materials"),
	TEXT("Useful for debugging, reattaches all materials. Optional parameter can be a material name (e.g. DecoStatue_Subsurface0_Inst)."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&ReattachMaterials)
	);


void ReattachComponents(const TArray<FString>& Args)
{
	if(Args.Num() != 1)
	{
		UE_LOGF(LogConsoleResponse, Warning, "Reattach.Components: missing class name parameter");
		return;
	}

	UE_LOGF(LogConsoleResponse, Display, "Reattach.Components:");

	UClass* Class=NULL;
	if( ParseObject<UClass>( *Args[0], TEXT("CLASS="), Class, nullptr ) &&
		Class->IsChildOf(UActorComponent::StaticClass()) )
	{
		for( FThreadSafeObjectIterator It(Class); It; ++It )
		{
			UActorComponent* ActorComponent = Cast<UActorComponent>(*It);
			if( ActorComponent )
			{
				UE_LOGF(LogConsoleResponse, Display, "   Component: %ls", 
					*ActorComponent->GetName());

				FComponentReregisterContext Reregister(ActorComponent);
			}
		}
		UE_LOGF(LogConsoleResponse, Display, "");
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Warning, "Reattach.Components: No objects with the class name '%ls' found", *Args[0]);
	}

}

FAutoConsoleCommand ReattachComponentsCmd(
	TEXT("Reattach.Components"),
	TEXT("Useful for debugging, reattaches all components. Parameter needs to be the class name.\n")
	TEXT(" Example: Reattach.Components class=SkeletalMeshComponent"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&ReattachComponents)
	);


#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)


#if !UE_BUILD_SHIPPING && PLATFORM_DESKTOP

static void ShowWidgetReflector()
{
	static const FName SlateReflectorModuleName("SlateReflector");
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>(SlateReflectorModuleName).DisplayWidgetReflector();
}

static void ShowTextureAtlasVisualizer()
{
	static const FName SlateReflectorModuleName("SlateReflector");
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>(SlateReflectorModuleName).DisplayTextureAtlasVisualizer();
}

static void ShowFontAtlasVisualizer()
{
	static const FName SlateReflectorModuleName("SlateReflector");
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>(SlateReflectorModuleName).DisplayFontAtlasVisualizer();
}

static FAutoConsoleCommand GShowWidgetReflector
(
	TEXT("WidgetReflector"),
	TEXT("Displays the Slate widget reflector"),
	FConsoleCommandDelegate::CreateStatic(ShowWidgetReflector)
);

static FAutoConsoleCommand GShowTextureAtlasReflector
(
	TEXT("TextureAtlasVisualizer"),
	TEXT("Displays the Slate texture atlas visualizer"),
	FConsoleCommandDelegate::CreateStatic(ShowTextureAtlasVisualizer)
);

static FAutoConsoleCommand GShowFontAtlasReflector
(
	TEXT("FontAtlasVisualizer"),
	TEXT("Displays the Slate font atlas visualizer"),
	FConsoleCommandDelegate::CreateStatic(ShowFontAtlasVisualizer)
);

static FAutoConsoleCommand GTranslationPicker
(
	TEXT("TranslationPicker"),
	TEXT("Displays the translation picker"),
	FConsoleCommandDelegate::CreateStatic([]()
	{
		static const FName TranslationEditorModuleName("TranslationEditor");
		FModuleManager::LoadModuleChecked<FTranslationEditorModule>(TranslationEditorModuleName).OpenTranslationPicker();
	})
);

#endif
