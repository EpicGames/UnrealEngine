// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/EngineMessage.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "Engine/Console.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/PlayerState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EngineMessage)

UEngineMessage::UEngineMessage(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}

void UEngineMessage::ClientReceive(const FClientReceiveData& ClientData) const
{
	//setup the local message string
	FString LocalMessageString = ClientData.MessageString;

	//if we have a local message string, then don't use the localized string
	if(LocalMessageString.IsEmpty()== true)
	{
		//This is the old GameMessage logic.
		switch (ClientData.MessageIndex)
		{
			case 1:
				if (ClientData.RelatedPlayerState_1 == NULL)
				{
					LocalMessageString = NewPlayerMessage;
				}
				else
				{
					LocalMessageString = FString::Printf(TEXT("%s%s"), *ClientData.RelatedPlayerState_1->GetPlayerName(), *EnteredMessage);
				}
				break;
			case 2:
				if (ClientData.RelatedPlayerState_1 == NULL)
				{
					LocalMessageString = TEXT("");
				}
				else
				{
					LocalMessageString = FString::Printf(TEXT("%s %s %s"), *ClientData.RelatedPlayerState_1->GetOldPlayerName(), *GlobalNameChange, *ClientData.RelatedPlayerState_1->GetPlayerName());
				}
				break;
			case 4:
				if (ClientData.RelatedPlayerState_1 == NULL)
				{
					LocalMessageString = TEXT("");
				}
				else
				{
					LocalMessageString = FString::Printf(TEXT("%s%s"), *ClientData.RelatedPlayerState_1->GetPlayerName(), *LeftMessage);
				}
				break;
			case 7:
				{
					LocalMessageString = MaxedOutMessage;
				}
				break;
			case 16:
				if (ClientData.RelatedPlayerState_1 == NULL)
				{
					LocalMessageString = NewSpecMessage;
				}
				else
				{
					LocalMessageString = FString::Printf(TEXT("%s%s"), *ClientData.RelatedPlayerState_1->GetPlayerName(), *SpecEnteredMessage);
				}
				break;
		}
	}

	//Engine messages are going to go out to the console.
	if(LocalMessageString.IsEmpty() == false &&
	   ClientData.LocalPC != NULL &&
	   Cast<ULocalPlayer>(ClientData.LocalPC->Player) != NULL && 
	   Cast<ULocalPlayer>(ClientData.LocalPC->Player)->ViewportClient != NULL && 
	   Cast<ULocalPlayer>(ClientData.LocalPC->Player)->ViewportClient->ViewportConsole != NULL)
	{
		Cast<ULocalPlayer>(ClientData.LocalPC->Player)->ViewportClient->ViewportConsole->OutputText( LocalMessageString );
	}
}

