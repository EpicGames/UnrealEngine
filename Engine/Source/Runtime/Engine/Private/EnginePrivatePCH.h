// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// From Core:
#include "CoreTypes.h"
#include "Misc/Exec.h"
#include "Misc/AssertionMacros.h"
#include "HAL/PlatformMisc.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "CoreFwd.h"
#include "Containers/ContainersFwd.h"
#include "UObject/UObjectHierarchyFwd.h"
#include "HAL/PlatformCrt.h"
#include "Containers/Array.h"
#include "HAL/UnrealMemory.h"
#include "Templates/IsPointer.h"
#include "HAL/PlatformMemory.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/MemoryBase.h"
#include "Misc/OutputDevice.h"
#include "Logging/LogVerbosity.h"
#include "Misc/VarArgs.h"
#include "HAL/PlatformAtomics.h"
#include "GenericPlatform/GenericPlatformAtomics.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/AndOrNot.h"
#include "Templates/IsArithmetic.h"
#include "Templates/IsPODType.h"
#include "Templates/IsTriviallyCopyConstructible.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/EnableIf.h"
#include "Templates/RemoveReference.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/IntegralConstant.h"
#include "Templates/IsClass.h"
#include "Traits/IsContiguousContainer.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/PlatformMath.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Templates/MemoryOps.h"
#include "Templates/IsTriviallyCopyAssignable.h"
#include "Math/NumericLimits.h"
#include "Serialization/Archive.h"
#include "Templates/IsEnumClass.h"
#include "HAL/PlatformProperties.h"
#include "GenericPlatform/GenericPlatformProperties.h"
#include "Misc/Compression.h"
#include "Misc/EngineVersionBase.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Templates/Less.h"
#include "Templates/Sorting.h"
#include "Containers/UnrealString.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "HAL/PlatformString.h"
#include "GenericPlatform/GenericPlatformStricmp.h"
#include "GenericPlatform/GenericPlatformString.h"
#include "Misc/Crc.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Map.h"
#include "Misc/StructBuilder.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Function.h"
#include "Templates/Decay.h"
#include "Templates/Invoke.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Containers/Set.h"
#include "Templates/TypeHash.h"
#include "Containers/SparseArray.h"
#include "Containers/ScriptArray.h"
#include "Containers/BitArray.h"
#include "Algo/Reverse.h"
#include "Math/Color.h"
#include "Misc/Parse.h"
#include "Logging/LogMacros.h"
#include "Logging/LogCategory.h"
#include "UObject/NameTypes.h"
#include "HAL/CriticalSection.h"
#include "Misc/Timespan.h"
#include "Containers/StringConv.h"
#include "UObject/UnrealNames.h"
#include "Templates/SharedPointer.h"
#include "CoreGlobals.h"
#include "HAL/PlatformTLS.h"
#include "GenericPlatform/GenericPlatformTLS.h"
#include "Delegates/Delegate.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Delegates/MulticastDelegateBase.h"
#include "Delegates/IDelegateInstance.h"
#include "Delegates/DelegateSettings.h"
#include "Delegates/DelegateBase.h"
#include "Delegates/IntegerSequence.h"
#include "Templates/Tuple.h"
#include "UObject/ScriptDelegates.h"
#include "CoreMinimal.h"
#include "Math/UnrealMath.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/IsIntegral.h"
#include "Templates/IsSigned.h"
#include "Templates/Greater.h"
#include "Math/ColorList.h"
#include "Math/IntRect.h"
#include "Math/IntPoint.h"
#include "Math/Vector2D.h"
#include "Math/Edge.h"
#include "Math/Vector.h"
#include "Misc/ByteSwap.h"
#include "Internationalization/Text.h"
#include "Containers/EnumAsByte.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"
#include "Templates/IsArray.h"
#include "Templates/RemoveExtent.h"
#include "Internationalization/Internationalization.h"
#include "Templates/UniqueObj.h"
#include "Math/IntVector.h"
#include "Math/CapsuleShape.h"
#include "Math/RangeSet.h"
#include "Math/Range.h"
#include "Misc/DateTime.h"
#include "Math/RangeBound.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Sphere.h"
#include "Math/Box.h"
#include "Math/OrientedBox.h"
#include "Math/Interval.h"
#include "Math/RotationAboutPointMatrix.h"
#include "Math/Matrix.h"
#include "Math/Vector4.h"
#include "Math/Plane.h"
#include "UObject/ObjectVersion.h"
#include "Math/Rotator.h"
#include "Math/VectorRegister.h"
#include "Math/Axis.h"
#include "Math/RotationTranslationMatrix.h"
#include "Math/ScaleRotationTranslationMatrix.h"
#include "Math/RotationMatrix.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/OrthoMatrix.h"
#include "Math/TranslationMatrix.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Math/Quat.h"
#include "Math/InverseRotationMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/MirrorMatrix.h"
#include "Math/ClipProjectionMatrix.h"
#include "Math/InterpCurve.h"
#include "Math/TwoVectors.h"
#include "Math/InterpCurvePoint.h"
#include "Math/CurveEdInterface.h"
#include "Math/Float16Color.h"
#include "Math/Float16.h"
#include "Math/Float32.h"
#include "Math/Vector2DHalf.h"
#include "Math/Transform.h"
#include "Math/ConvexHull2d.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSingleton.h"
#include "HAL/TlsAutoCleanup.h"
#include "HAL/PlatformTime.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "Stats/Stats.h"
#include "Containers/LockFreeList.h"
#include "Misc/NoopCounter.h"
#include "Containers/ChunkedArray.h"
#include "Containers/IndirectArray.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/OutputDeviceRedirector.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "Misc/Guid.h"
#include "Math/RandomStream.h"
#include "Containers/List.h"
#include "Misc/CoreMisc.h"
#include "HAL/Event.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"
#include "Misc/NetworkGuid.h"
#include "HAL/IConsoleManager.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "Misc/MemStack.h"
#include "Templates/RefCounting.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/Attribute.h"
#include "Templates/ScopedCallback.h"
#include "Misc/CoreStats.h"
#include "Modules/ModuleInterface.h"
#include "HAL/PlatformProcess.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "GenericPlatform/ICursor.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "Misc/IQueuedWork.h"
#include "Misc/QueuedThreadPool.h"
#include "Async/AsyncWork.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "Math/TransformCalculus.h"
#include "Math/TransformCalculus2D.h"
#include "Serialization/BufferReader.h"
#include "Misc/SecureHash.h"
#include "Containers/StaticArray.h"
#include "Async/Future.h"
#include "Serialization/ArchiveProxy.h"
#include "UObject/DebugSerializationFlags.h"
#include "Math/SHMath.h"
#include "Misc/ScopedEvent.h"
#include "Containers/ResourceArray.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Modules/ModuleManager.h"
#include "Modules/Boilerplate/ModuleBoilerplate.h"
#include "Misc/BufferedOutputDevice.h"
#include "HAL/ThreadSafeBool.h"
#include "Serialization/CustomVersion.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "Misc/ObjectThumbnail.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Misc/App.h"
#include "Containers/Queue.h"
#include "GenericPlatform/IInputInterface.h"
#include "Misc/EngineVersion.h"
#include "Serialization/MemoryArchive.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "Internationalization/GatherableTextData.h"
#include "Serialization/MemoryWriter.h"
#include "HAL/Runnable.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/ConfigCacheIni.h"
#include "Containers/ArrayView.h"

// From CoreUObject:
#include "UObject/ObjectMacros.h"
#include "UObject/Script.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectMarks.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GeneratedCppIncludes.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/CoreNative.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/CoreNetTypes.h"
#include "Templates/Casts.h"
#include "UObject/PersistentObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyTag.h"
#include "Serialization/SerializedPropertyScope.h"
#include "Templates/SubclassOf.h"
#include "UObject/CoreNet.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Stack.h"
#include "UObject/Interface.h"
#include "Misc/WorldCompositionUtility.h"
#include "UObject/GCObject.h"
#include "Serialization/BulkData.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Online/CoreOnline.h"
#include "UObject/TextProperty.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectResource.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Linker.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ConstructorHelpers.h"

// From InputCore:
#include "InputCoreTypes.h"

// From SlateCore:
#include "Types/SlateEnums.h"
#include "Styling/SlateColor.h"
#include "Styling/WidgetStyle.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Rendering/SlateRenderTransform.h"
#include "Input/Events.h"
#include "Widgets/SWidget.h"
#include "Input/CursorReply.h"
#include "Input/ReplyBase.h"
#include "Input/Reply.h"
#include "Input/DragAndDrop.h"
#include "Input/NavigationReply.h"
#include "Input/PopupMethodReply.h"
#include "Types/ISlateMetaData.h"
#include "Layout/ArrangedWidget.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Layout/LayoutGeometry.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SNullWidget.h"
#include "SlotBase.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"
#include "Widgets/SOverlay.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/CompositeFont.h"
#include "Sound/SlateSound.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Rendering/RenderingCommon.h"
#include "Styling/SlateTypes.h"
#include "Styling/ISlateStyle.h"
#include "Styling/StyleDefaults.h"
#include "Brushes/SlateNoResource.h"
#include "Types/SlateStructs.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Styling/SlateWidgetStyleContainerInterface.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SWindow.h"
#include "Animation/CurveSequence.h"
#include "Animation/CurveHandle.h"
#include "Fonts/ShapedTextFwd.h"
#include "Fonts/FontProviderInterface.h"

// From Slate:
#include "SlateFwd.h"
#include "Framework/Commands/InputChord.h"

// From RHI:
#include "RHIDefinitions.h"
#include "RHI.h"
#include "RHIStaticStates.h"

// From RenderCore:
#include "RenderCommandFence.h"
#include "RenderResource.h"
#include "RenderingThread.h"
#include "UniformBuffer.h"
#include "PackedNormal.h"
#include "RenderUtils.h"
#include "ShaderParameters.h"
#include "ShaderCore.h"
#include "Shader.h"
#include "VertexFactory.h"

// From AssetRegistry:
#include "AssetRegistry/AssetData.h"

// From Engine:
#include "EngineLogs.h"
#include "Engine/EngineTypes.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineBaseTypes.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Components/ActorComponent.h"
#include "ComponentInstanceDataCache.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Level.h"
#include "Engine/MaterialMerging.h"
#include "EngineDefines.h"
#include "CollisionQueryParams.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavQueryFilter.h"
#include "WorldCollision.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintCore.h"
#include "SceneTypes.h"
#include "GameFramework/Pawn.h"
#include "Engine/LatentActionManager.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "Engine/PendingNetGame.h"
#include "PixelFormat.h"
#include "HitProxies.h"
#include "Engine/BlendableInterface.h"
#include "UnrealClient.h"
#include "Engine/Scene.h"
#include "ShowFlags.h"
#include "Engine/ScriptViewportClient.h"
#include "Engine/GameViewportClient.h"
#include "Engine/ViewportSplitScreen.h"
#include "Engine/TitleSafeZone.h"
#include "Engine/GameViewportDelegates.h"
#include "Engine/DebugDisplayProperty.h"
#include "PhysxUserData.h"
#include "PhysicsEngine/BodyInstance.h"
#include "SceneInterface.h"
#include "Curves/KeyHandle.h"
#include "Curves/IndexedCurve.h"
#include "DebugViewModeHelpers.h"
#include "SceneView.h"
#include "ConvexVolume.h"
#include "FinalPostProcessSettings.h"
#include "BlendableManager.h"
#include "Curves/RichCurve.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "Engine/TextureStreamingTypes.h"
#include "Components/PrimitiveComponent.h"
#include "Components.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveUniformShaderParameters.h"
#include "MaterialShared.h"
#include "StaticParameterSet.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/TextureDefines.h"
#include "Curves/CurveOwnerInterface.h"
#include "Engine/Texture.h"
#include "Curves/CurveBase.h"
#include "TextureResource.h"
#include "Curves/CurveFloat.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "Engine/Brush.h"
#include "StaticBoundShaderState.h"
#include "BatchedElements.h"
#include "GameFramework/Volume.h"
#include "EngineGlobals.h"
#include "Audio.h"
#include "AudioDevice.h"
#include "Sound/SoundAttenuation.h"
#include "IAudioExtensionPlugin.h"
#include "Components/MeshComponent.h"
#include "BoneIndices.h"
#include "ReferenceSkeleton.h"
#include "Sound/AudioVolume.h"
#include "SceneUtils.h"
#include "GPUSkinPublicDefs.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimLinkableElement.h"
#include "MeshBatch.h"
#include "Animation/PreviewAssetAttachComponent.h"
#include "BoneContainer.h"
#include "LocalVertexFactory.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/TextureLightProfile.h"
#include "SceneManagement.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/SmartName.h"
#include "Animation/Skeleton.h"
#include "AnimInterpFilter.h"
#include "GameFramework/Info.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimCurveTypes.h"
#include "Engine/DeveloperSettings.h"
#include "GameFramework/DamageType.h"
#include "GameFramework/WorldSettings.h"
#include "RawIndexBuffer.h"
#include "Components/SkinnedMeshComponent.h"
#include "Camera/CameraTypes.h"
#include "Components/InputComponent.h"
#include "GameFramework/Controller.h"
#include "GameFramework/OnlineReplStructs.h"
#include "BodySetupEnums.h"
#include "Animation/AnimSequenceBase.h"
#include "Math/GenericOctreePublic.h"
#include "Camera/PlayerCameraManager.h"
#include "Camera/CameraShakeBase.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "Materials/Material.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialFunction.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerMuteList.h"
#include "Animation/AnimSequence.h"
#include "AI/NavigationSystemBase.h"
#include "Sound/SoundConcurrency.h"
#include "ClothSimData.h"
#include "DynamicMeshBuilder.h"
#include "EngineStats.h"
#include "Components/SkeletalMeshComponent.h"
#include "SingleAnimationPlayData.h"
#include "Engine/StaticMesh.h"
#include "Engine/CollisionProfile.h"
#include "Components/AudioComponent.h"
#include "Animation/AnimStats.h"
#include "PhysicsPublic.h"
#include "CustomBoneIndexArray.h"
#include "EngineUtils.h"
#include "BonePose.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/ShapeElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/Player.h"
#include "Engine/Font.h"
#include "Engine/FontImportOptions.h"
#include "CanvasTypes.h"
#include "Components/StaticMeshComponent.h"
#include "ContentStreaming.h"
#include "GameFramework/MovementComponent.h"
#include "CanvasItem.h"
#include "Model.h"
#include "UnrealEngine.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "ParticleHelper.h"
#include "MeshParticleVertexFactory.h"
#include "ParticleVertexFactory.h"
#include "AlphaBlend.h"
#include "Animation/AnimCompositeBase.h"
#include "Particles/ParticleSystem.h"
#include "Particles/Emitter.h"
#include "TimerManager.h"
