// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineService.h"
#include "Engine/World.h"
#include "Misc/App.h"
#include "Engine/Engine.h"
#include "EngineServiceMessages.h"
#include "MessageEndpointBuilder.h"


DEFINE_LOG_CATEGORY_STATIC(LogEngineService, Log, All)


/* FEngineService structors
 *****************************************************************************/

FEngineService::FEngineService()
{
	// initialize messaging
	MessageEndpoint = FMessageEndpoint::Builder("FEngineService")
		.Handling<FEngineServiceAuthDeny>(this, &FEngineService::HandleAuthDenyMessage)
		.Handling<FEngineServiceAuthGrant>(this, &FEngineService::HandleAuthGrantMessage)
		.Handling<FEngineServiceExecuteCommand>(this, &FEngineService::HandleExecuteCommandMessage)
		.Handling<FEngineServicePing>(this, &FEngineService::HandlePingMessage)
		.Handling<FEngineServiceTerminate>(this, &FEngineService::HandleTerminateMessage)
		.ReceivingOnThread(ENamedThreads::GameThread);

	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Subscribe<FEngineServicePing>();
	}
}


/* FEngineService implementation
 *****************************************************************************/

void FEngineService::SendNotification( const TCHAR* NotificationText, const FMessageAddress& Recipient )
{
	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FEngineServiceNotification>(NotificationText, FPlatformTime::Seconds() - GStartTime), Recipient);
	}
}


void FEngineService::SendPong( const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (MessageEndpoint.IsValid())
	{
		FEngineServicePong* Message = FMessageEndpoint::MakeMessage<FEngineServicePong>();

		Message->EngineVersion = FNetworkVersion::GetNetworkCompatibleChangelist();
		Message->InstanceId = FApp::GetInstanceId();
		Message->SessionId = FApp::GetSessionId();

		if (GEngine == nullptr)
		{
			Message->InstanceType = TEXT("Unknown");
		}
		else if (IsRunningDedicatedServer())
		{
			Message->InstanceType = TEXT("Server");
		}
		else if (IsRunningCommandlet())
		{
			Message->InstanceType = TEXT("Commandlet");
		}
		else if (GEngine->IsEditor())
		{
			Message->InstanceType = TEXT("Editor");
		}
		else if ( IsRunningGame() )
		{
			Message->InstanceType = TEXT("Game");
		}
		else
		{
			Message->InstanceType = TEXT("Other");
		}

		FWorldContext const* ContextToUse = nullptr;
		if (GEngine)
		{
			// TODO: Should we be iterating here and sending a message for each context?

			// We're going to look through the WorldContexts and pull any Game context we find
			// If there isn't a Game context, we'll take the first PIE we find
			// and if none of those we'll use an Editor
			for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
			{
				if (WorldContext.WorldType == EWorldType::Game)
				{
					ContextToUse = &WorldContext;
					break;
				}
				else if (WorldContext.WorldType == EWorldType::PIE && (ContextToUse == nullptr || ContextToUse->WorldType != EWorldType::PIE))
				{
					ContextToUse = &WorldContext;
				}
				else if (WorldContext.WorldType == EWorldType::Editor && ContextToUse == nullptr)
				{	
					ContextToUse = &WorldContext;
				}
			}
		}

		if (ContextToUse != nullptr && ContextToUse->World() != nullptr)
		{
			Message->CurrentLevel = ContextToUse->World()->GetMapName();
			Message->HasBegunPlay = ContextToUse->World()->HasBegunPlay();
			Message->WorldTimeSeconds = ContextToUse->World()->TimeSeconds;
		}

		MessageEndpoint->Send(Message, Context->GetSender());
	}
}


/* FEngineService callbacks
 *****************************************************************************/

void FEngineService::HandleAuthGrantMessage( const FEngineServiceAuthGrant& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (FApp::IsAuthorizedUser(Message.UserName))
	{
		FApp::AuthorizeUser(Message.UserToGrant);

		UE_LOGF(LogEngineService, Log, "%ls granted remote access to user %ls.", *Message.UserName, *Message.UserToGrant);
	}
	else
	{
		SendNotification(TEXT("You are not authorized to grant or deny remote access to other users."), Context->GetSender());
	}
}


void FEngineService::HandleAuthDenyMessage( const FEngineServiceAuthDeny& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (FApp::IsAuthorizedUser(Message.UserName))
	{
		FApp::DenyUser(Message.UserToDeny);

		UE_LOGF(LogEngineService, Log, "%ls removed remote access from user %ls.", *Message.UserName, *Message.UserToDeny);
	}
	else
	{
		SendNotification(TEXT("You are not authorized to grant or deny remote access to other users."), Context->GetSender());
	}
}


void FEngineService::HandleExecuteCommandMessage( const FEngineServiceExecuteCommand& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (FApp::IsAuthorizedUser(Message.UserName))
	{
		if (GEngine != nullptr)
		{
			GEngine->DeferredCommands.Add(Message.Command);

			UE_LOGF(LogEngineService, Log, "%ls executed the remote command: %ls", *Message.UserName, *Message.Command);
		}
		else
		{
			SendNotification(TEXT("The command could not be executed because the Engine has not started up yet."), Context->GetSender());
		}
	}
	else
	{
		SendNotification(TEXT("You are not authorized to execute console commands."), Context->GetSender());
	}
}


void FEngineService::HandlePingMessage( const FEngineServicePing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	SendPong(Context);
}


void FEngineService::HandleTerminateMessage( const FEngineServiceTerminate& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (FApp::IsAuthorizedUser(Message.UserName))
	{
		if (GEngine != nullptr)
		{
			UE_LOGF(LogEngineService, Log, "%ls terminated this instance remotely.", *Message.UserName);

			if (GEngine->IsEditor())
			{
				GEngine->Exec( nullptr, TEXT("QUIT_EDITOR"), *GLog);
			}
			else
			{
				GEngine->Exec( nullptr, TEXT("QUIT"), *GLog);
			}
		}
		else
		{
			SendNotification(TEXT("Termination failed because the Engine has not started up yet or is unavailable."), Context->GetSender());
		}
	}
	else
	{
		SendNotification(TEXT("You are not authorized to terminate this instance."), Context->GetSender());
	}
}
