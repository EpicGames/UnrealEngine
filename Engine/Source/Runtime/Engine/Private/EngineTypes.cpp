// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/EngineTypes.h"
#include "UObject/UnrealType.h"
#include "GameFramework/Actor.h"
#include "Engine/CollisionProfile.h"
#include "Hash/xxhash.h"
#include "Serialization/MemoryHasher.h"

#if WITH_EDITOR
#include "Engine/Texture2D.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(EngineTypes)

FAttachmentTransformRules FAttachmentTransformRules::KeepRelativeTransform(EAttachmentRule::KeepRelative, false);
FAttachmentTransformRules FAttachmentTransformRules::KeepWorldTransform(EAttachmentRule::KeepWorld, false);
FAttachmentTransformRules FAttachmentTransformRules::SnapToTargetNotIncludingScale(EAttachmentRule::SnapToTarget, EAttachmentRule::SnapToTarget, EAttachmentRule::KeepWorld, false);
FAttachmentTransformRules FAttachmentTransformRules::SnapToTargetIncludingScale(EAttachmentRule::SnapToTarget, false);

FDetachmentTransformRules FDetachmentTransformRules::KeepRelativeTransform(EDetachmentRule::KeepRelative, true);
FDetachmentTransformRules FDetachmentTransformRules::KeepWorldTransform(EDetachmentRule::KeepWorld, true);

UEngineBaseTypes::UEngineBaseTypes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UEngineTypes::UEngineTypes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

ECollisionChannel UEngineTypes::ConvertToCollisionChannel(ETraceTypeQuery TraceType)
{
	return UCollisionProfile::Get()->ConvertToCollisionChannel(true, (int32)TraceType);
}

ECollisionChannel UEngineTypes::ConvertToCollisionChannel(EObjectTypeQuery ObjectType)
{
	return UCollisionProfile::Get()->ConvertToCollisionChannel(false, (int32)ObjectType);
}

EObjectTypeQuery UEngineTypes::ConvertToObjectType(ECollisionChannel CollisionChannel)
{
	return UCollisionProfile::Get()->ConvertToObjectType(CollisionChannel);
}

ETraceTypeQuery UEngineTypes::ConvertToTraceType(ECollisionChannel CollisionChannel)
{
	return UCollisionProfile::Get()->ConvertToTraceType(CollisionChannel);
}

FLightmassDebugOptions::FLightmassDebugOptions()
	: bDebugMode(false)
	, bStatsEnabled(false)
	, bGatherBSPSurfacesAcrossComponents(true)
	, CoplanarTolerance(0.001f)
	, bUseImmediateImport(true)
	, bImmediateProcessMappings(true)
	, bSortMappings(true)
	, bDumpBinaryFiles(false)
	, bDebugMaterials(false)
	, bPadMappings(true)
	, bDebugPaddings(false)
	, bOnlyCalcDebugTexelMappings(false)
	, bUseRandomColors(false)
	, bColorBordersGreen(false)
	, bColorByExecutionTime(false)
	, ExecutionTimeDivisor(15.0f)
{}


UActorComponent* FBaseComponentReference::ExtractComponent(AActor* SearchActor) const 
{
	UActorComponent* Result = nullptr;

	// Component is specified directly, use that
	if(OverrideComponent.IsValid())
	{
		Result = OverrideComponent.Get();
	}
	else
	{
		if(SearchActor)
		{
			if(ComponentProperty != NAME_None)
			{
				FObjectPropertyBase* ObjProp = FindFProperty<FObjectPropertyBase>(SearchActor->GetClass(), ComponentProperty);
				if(ObjProp != NULL)
				{
					// .. and return the component that is there
					Result = Cast<UActorComponent>(ObjProp->GetObjectPropertyValue_InContainer(SearchActor));
				}
			}
			else if (!PathToComponent.IsEmpty())
			{
				Result = FindObject<UActorComponent>(SearchActor, *PathToComponent);
			}
			else
			{
				Result = SearchActor->GetRootComponent();
			}
		}
	}

	return Result;
}

UActorComponent* FComponentReference::GetComponent(AActor* OwningActor) const
{
	AActor* SearchActor = (OtherActor.IsValid()) ? OtherActor.Get() : OwningActor;
	return ExtractComponent(SearchActor);
}

UActorComponent* FSoftComponentReference::GetComponent(AActor* OwningActor) const
{
	AActor* SearchActor = (OtherActor.IsValid()) ? OtherActor.Get() : OwningActor;
	return ExtractComponent(SearchActor);
}

bool FSoftComponentReference::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName ComponentReferenceContextName("ComponentReference");
	if (Tag.GetType().IsStruct(ComponentReferenceContextName))
	{
		FComponentReference Reference;
		FComponentReference::StaticStruct()->SerializeItem(Slot, &Reference, nullptr);
		if (Reference.OtherActor.IsValid())
		{
			OtherActor = Reference.OtherActor.Get();
			ComponentProperty = Reference.ComponentProperty;
			PathToComponent = Reference.PathToComponent;
		}
		return true;
	}

	return false;
}

const TCHAR* LexToString(const EWorldType::Type Value)
{
	switch (Value)
	{
	case EWorldType::Type::Editor:
		return TEXT("Editor");
	case EWorldType::Type::EditorPreview:
		return TEXT("EditorPreview");
	case EWorldType::Type::Game:
		return TEXT("Game");
	case EWorldType::Type::GamePreview:
		return TEXT("GamePreview");
	case EWorldType::Type::GameRPC:
		return TEXT("GameRPC");
	case EWorldType::Type::Inactive:
		return TEXT("Inactive");
	case EWorldType::Type::PIE:
		return TEXT("PIE");
	case EWorldType::Type::None:
		return TEXT("None");
	default:
		return TEXT("Unknown");
	}
}

#if WITH_EDITOR

void SerializeNaniteSettingsForDDC(FArchive& Ar, FMeshNaniteSettings& NaniteSettings, bool bIsNaniteForceEnabled)
{
	bool bIsEnabled = NaniteSettings.bEnabled || bIsNaniteForceEnabled;

	// Note: this serializer is only used to build the mesh DDC key, no versioning is required
	FArchive_Serialize_BitfieldBool(Ar, bIsEnabled);
	FArchive_Serialize_BitfieldBool(Ar, NaniteSettings.bExplicitTangents);
	FArchive_Serialize_BitfieldBool(Ar, NaniteSettings.bLerpUVs);
	Ar << NaniteSettings.ShapePreservation;
	Ar << NaniteSettings.PositionPrecision;
	Ar << NaniteSettings.NormalPrecision;
	Ar << NaniteSettings.TangentPrecision;
	Ar << NaniteSettings.BoneWeightPrecision;
	Ar << NaniteSettings.TargetMinimumResidencyInKB;
	Ar << NaniteSettings.KeepPercentTriangles;
	Ar << NaniteSettings.TrimRelativeError;
	Ar << NaniteSettings.FallbackTarget;
	Ar << NaniteSettings.FallbackPercentTriangles;
	Ar << NaniteSettings.FallbackRelativeError;
	Ar << NaniteSettings.MaxEdgeLengthFactor;
	Ar << NaniteSettings.DisplacementUVChannel;
	
	{
		TMemoryHasher<FXxHash64Builder, FXxHash64> HashAr;
		HashAr.SetIsPersistent(true);
		NaniteSettings.NaniteAssemblyData.SerializeForDDC(HashAr);
		
		uint64 AssemblyDataHash = HashAr.Finalize().Hash;
		Ar << AssemblyDataHash;
	}

	if (NaniteSettings.ShapePreservation == ENaniteShapePreservation::Voxelize)
	{
		Ar << NaniteSettings.NumRays;
		Ar << NaniteSettings.VoxelLevel;
		Ar << NaniteSettings.RayBackUp;
		FArchive_Serialize_BitfieldBool(Ar, NaniteSettings.bSeparable);
		FArchive_Serialize_BitfieldBool(Ar, NaniteSettings.bVoxelNDF);
		FArchive_Serialize_BitfieldBool(Ar, NaniteSettings.bVoxelOpacity);
	}

	for (auto& DisplacementMap : NaniteSettings.DisplacementMaps)
	{
		if (IsValid(DisplacementMap.Texture))
		{
			FGuid TextureId = DisplacementMap.Texture->Source.GetId();
			Ar << TextureId;
			Ar << DisplacementMap.Texture->AddressX;
			Ar << DisplacementMap.Texture->AddressY;
		}

		Ar << DisplacementMap.Magnitude;
		Ar << DisplacementMap.Center;
	}
}

void SerializeRayTracingProxySettingsForDDC(FArchive& Ar, FMeshRayTracingProxySettings& Settings)
{
	FArchive_Serialize_BitfieldBool(Ar, Settings.bEnabled);
	Ar << Settings.FallbackTarget;
	Ar << Settings.FallbackPercentTriangles;
	Ar << Settings.FallbackRelativeError;
	Ar << Settings.FoliageOverOcclusionBias;
}

#endif

/// @endcond
