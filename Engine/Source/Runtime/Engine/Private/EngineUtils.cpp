// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
=============================================================================*/
#include "EngineUtils.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "GameFramework/Pawn.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "Misc/EngineVersion.h"
#include "GameFramework/PlayerController.h"
#include "GenericPlatform/ICursor.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "Engine/Console.h"
#include "Engine/Texture.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Misc/PathViews.h"
#include "WorldPartition/WorldPartitionRuntimeCellInterface.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "Interfaces/ITargetPlatform.h"


DEFINE_LOG_CATEGORY_STATIC(LogEngineUtils, Log, All);

IMPLEMENT_HIT_PROXY(HActor,HHitProxy)
IMPLEMENT_HIT_PROXY(HBSPBrushVert,HHitProxy);
IMPLEMENT_HIT_PROXY(HStaticMeshVert,HHitProxy);
IMPLEMENT_HIT_PROXY(HTranslucentActor,HActor)

#define LOCTEXT_NAMESPACE "EngineUtils"

void HActor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Actor);
	Collector.AddReferencedObject(PrimComponent);
}

EMouseCursor::Type HActor::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}

FTypedElementHandle HActor::GetElementHandle() const
{
#if WITH_EDITOR
	if (PrimComponent)
	{
		return UEngineElementsLibrary::AcquireEditorComponentElementHandle(PrimComponent);
	}
	if (Actor)
	{
		return UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor);
	}
#endif	// WITH_EDITOR
	return FTypedElementHandle();
}

bool HActor::AlwaysAllowsTranslucentPrimitives() const
{
#if WITH_EDITOR
	return PrimComponent && PrimComponent->bAlwaysAllowTranslucentSelect;
#else
	return false;
#endif
}

EMouseCursor::Type HTranslucentActor::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}

bool HTranslucentActor::AlwaysAllowsTranslucentPrimitives() const
{
	return true;
}

#if !UE_BUILD_SHIPPING
FContentComparisonHelper::FContentComparisonHelper()
{
	const FConfigSection* RefTypes = GConfig->GetSection(TEXT("ContentComparisonReferenceTypes"), false, GEngineIni);
	if (RefTypes != NULL)
	{
		for( FConfigSectionMap::TConstIterator It(*RefTypes); It; ++It )
		{
			const FString& RefType = It.Value().GetValue();
			ReferenceClassesOfInterest.Add(RefType, true);
			UE_LOGF(LogEngineUtils, Log, "Adding class of interest: %ls", *RefType);
		}
	}
}

FContentComparisonHelper::~FContentComparisonHelper()
{
}

bool FContentComparisonHelper::CompareClasses(const FString& InBaseClassName, int32 InRecursionDepth)
{
	TArray<FString> EmptyIgnoreList;
	return CompareClasses(InBaseClassName, EmptyIgnoreList, InRecursionDepth);
}

bool FContentComparisonHelper::CompareClasses(const FString& InBaseClassName, const TArray<FString>& InBaseClassesToIgnore, int32 InRecursionDepth)
{
	TMap<FString,TArray<FContentComparisonAssetInfo> > ClassToAssetsMap;

	UClass* TheClass = FindFirstObject<UClass>(*InBaseClassName, EFindFirstObjectOptions::ExactClass, ELogVerbosity::Warning, TEXT("FContentComparisonHelper::CompareClasses"));
	if (TheClass != NULL)
	{
		TArray<UClass*> IgnoreBaseClasses;
		for (int32 IgnoreIdx = 0; IgnoreIdx < InBaseClassesToIgnore.Num(); IgnoreIdx++)
		{
			UClass* IgnoreClass = FindFirstObject<UClass>(*(InBaseClassesToIgnore[IgnoreIdx]), EFindFirstObjectOptions::ExactClass, ELogVerbosity::Warning, TEXT("FContentComparisonHelper::CompareClasses"));
			if (IgnoreClass != NULL)
			{
				IgnoreBaseClasses.Add(IgnoreClass);
			}
		}

		for( TObjectIterator<UClass> It; It; ++It )
		{
			UClass* TheAssetClass = *It;
			if ((TheAssetClass->IsChildOf(TheClass) == true) && 
				(TheAssetClass->HasAnyClassFlags(CLASS_Abstract) == false))
			{
				bool bSkipIt = false;
				for (int32 CheckIdx = 0; CheckIdx < IgnoreBaseClasses.Num(); CheckIdx++)
				{
					UClass* CheckClass = IgnoreBaseClasses[CheckIdx];
					if (TheAssetClass->IsChildOf(CheckClass) == true)
					{
// 						UE_LOGF(LogEngineUtils, Warning, "Skipping class derived from other content comparison class...");
// 						UE_LOGF(LogEngineUtils, Warning, "\t%ls derived from %ls", *TheAssetClass->GetFullName(), *CheckClass->GetFullName());
						bSkipIt = true;
					}
				}
				if (bSkipIt == false)
				{
					TArray<FContentComparisonAssetInfo>* AssetList = ClassToAssetsMap.Find(TheAssetClass->GetFullName());
					if (AssetList == NULL)
					{
						TArray<FContentComparisonAssetInfo> TempAssetList;
						ClassToAssetsMap.Add(TheAssetClass->GetFullName(), TempAssetList);
						AssetList = ClassToAssetsMap.Find(TheAssetClass->GetFullName());
					}
					check(AssetList);

					// Serialize object with reference collector.
					const int32 MaxRecursionDepth = 6;
					InRecursionDepth = FMath::Clamp<int32>(InRecursionDepth, 1, MaxRecursionDepth);
					TMap<UObject*,bool> RecursivelyGatheredReferences;
					RecursiveObjectCollection(TheAssetClass, 0, InRecursionDepth, RecursivelyGatheredReferences);

					// Add them to the asset list
					for (TMap<UObject*,bool>::TIterator GatheredIt(RecursivelyGatheredReferences); GatheredIt; ++GatheredIt)
					{
						UObject* Object = GatheredIt.Key();
						if (Object)
						{
							bool bAddIt = true;
							if (ReferenceClassesOfInterest.Num() > 0)
							{
								FString CheckClassName = Object->GetClass()->GetName();
								if (ReferenceClassesOfInterest.Find(CheckClassName) == NULL)
								{
									bAddIt = false;
								}
							}
							if (bAddIt == true)
							{
								int32 NewIndex = AssetList->AddZeroed();
								FContentComparisonAssetInfo& Info = (*AssetList)[NewIndex];
								Info.AssetName = Object->GetFullName();
								Info.ResourceSize = Object->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
							}
						}
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogEngineUtils, Warning, "Failed to find class: %ls", *InBaseClassName);
		return false;
	}

#if 0
	// Log them all out
	UE_LOGF(LogEngineUtils, Log, "CompareClasses on %ls", *InBaseClassName);
	for (TMap<FString,TArray<FContentComparisonAssetInfo>>::TIterator It(ClassToAssetsMap); It; ++It)
	{
		FString ClassName = It.Key();
		TArray<FContentComparisonAssetInfo>& AssetList = It.Value();

		UE_LOGF(LogEngineUtils, Log, "\t%ls", *ClassName);
		for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); AssetIdx++)
		{
			FContentComparisonAssetInfo& Info = AssetList(AssetIdx);

			UE_LOGF(LogEngineUtils, Log, "\t\t%ls,%f", *(Info.AssetName), Info.ResourceSize/1024.0f);
		}
	}
#endif

#if ALLOW_DEBUG_FILES
	// Write out a CSV file
	FString CurrentTime = FDateTime::Now().ToString();
	FString Platform(FPlatformProperties::PlatformName());

	FString BaseCSVName = (
		FString("ContentComparison/") + 
		FString::Printf(TEXT("ContentCompare-%s/"), *FEngineVersion::Current().ToString()) +
		FString::Printf(TEXT("%s"), *InBaseClassName)
		);

	// Handle file name length on consoles... 
	FString EditedBaseClassName = InBaseClassName;
	FString TimeString = *FDateTime::Now().ToString();
	FString CheckLenName = FString::Printf(TEXT("%s-%s.csv"),*InBaseClassName,*TimeString);
	if (CheckLenName.Len() > FPlatformMisc::GetMaxPathLength())
	{
		while (CheckLenName.Len() > FPlatformMisc::GetMaxPathLength())
		{
			EditedBaseClassName = EditedBaseClassName.Right(EditedBaseClassName.Len() - 1);
			CheckLenName = FString::Printf(TEXT("%s-%s.csv"),*EditedBaseClassName,*TimeString);
		}
		BaseCSVName = (
			FString("ContentComparison/") + 
			FString::Printf(TEXT("ContentCompare-%s/"), *FEngineVersion::Current().ToString()) +
			FString::Printf(TEXT("%s"), *EditedBaseClassName)
			);
	}

	FDiagnosticTableViewer* AssetTable = new FDiagnosticTableViewer(
			*FDiagnosticTableViewer::GetUniqueTemporaryFilePath(*BaseCSVName), true);
	if ((AssetTable != NULL) && (AssetTable->OutputStreamIsValid() == true))
	{
		// Fill in the header row
		AssetTable->AddColumn(TEXT("Class"));
		AssetTable->AddColumn(TEXT("Asset"));
		AssetTable->AddColumn(TEXT("ResourceSize(kB)"));
		AssetTable->CycleRow();

		// Fill it in
		for (TMap<FString,TArray<FContentComparisonAssetInfo> >::TIterator It(ClassToAssetsMap); It; ++It)
		{
			FString ClassName = It.Key();
			TArray<FContentComparisonAssetInfo>& AssetList = It.Value();

			AssetTable->AddColumn(*ClassName);
			AssetTable->CycleRow();
			for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); AssetIdx++)
			{
				FContentComparisonAssetInfo& Info = AssetList[AssetIdx];

				AssetTable->AddColumn(TEXT(""));
				AssetTable->AddColumn(*(Info.AssetName));
				AssetTable->AddColumn(TEXT("%f"), Info.ResourceSize/1024.0f);
				AssetTable->CycleRow();
			}
		}
	}
	else if (AssetTable != NULL)
	{
		// Created the class, but it failed to open the output stream.
		UE_LOGF(LogEngineUtils, Warning, "Failed to open output stream in asset table!");
	}

	if (AssetTable != NULL)
	{
		// Close it and kill it
		AssetTable->Close();
		delete AssetTable;
	}
#endif

	return true;
}

void FContentComparisonHelper::RecursiveObjectCollection(UObject* InStartObject, int32 InCurrDepth, int32 InMaxDepth, TMap<UObject*,bool>& OutCollectedReferences)
{
	// Serialize object with reference collector.
	TArray<UObject*> LocalCollectedReferences;
	FReferenceFinder ObjectReferenceCollector( LocalCollectedReferences, NULL, false, true, true, true );
	ObjectReferenceCollector.FindReferences( InStartObject );

	if (InCurrDepth < InMaxDepth)
	{
		InCurrDepth++;
		for (int32 ObjRefIdx = 0; ObjRefIdx < LocalCollectedReferences.Num(); ObjRefIdx++)
		{
			UObject* InnerObject = LocalCollectedReferences[ObjRefIdx];
			if ((InnerObject != NULL) &&
				(InnerObject->IsA(UFunction::StaticClass()) == false) &&
				(InnerObject->IsA(UPackage::StaticClass()) == false)
				)
			{
				OutCollectedReferences.Add(InnerObject, true);
				RecursiveObjectCollection(InnerObject, InCurrDepth, InMaxDepth, OutCollectedReferences);
			}
		}
		InCurrDepth--;
	}
}
#endif

bool EngineUtils::FindOrLoadAssetsByPath(const FString& Path, TArray<UObject*>& OutAssets, EAssetToLoad Type)
{
	if ( !FPackageName::IsValidPath(Path))
	{
		return false;
	}

	using FPackageNames = TSet<FName>;

	auto GetPackageNamesFromPath = [](const FString& InPath, FPackageNames& OutPackageNames)
	{
		// There is no filesystem support for packages when using the I/O dispatcher
		if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::LoadModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
		{
			IAssetRegistry& AssetRegistry = AssetRegistryModule->Get();

			TArray<FAssetData> Assets;
			AssetRegistry.GetAssetsByPath(FName(*InPath), Assets, true);

			for (const FAssetData& Asset : Assets)
			{
				// Cull packages containing maps
				if ((Asset.PackageFlags & PKG_ContainsMap) != PKG_ContainsMap)
				{
					OutPackageNames.Emplace(Asset.PackageName);
				}
			}
		}

		// Convert the package path to a filename with no extension (directory)
		const FString FilePath = FPackageName::LongPackageNameToFilename(InPath);

		// Gather the package files in that directory and subdirectories
		TArray<FString> Filenames;
		FPackageName::FindPackagesInDirectory(Filenames, FilePath);

		// Cull out map files
		for (const FString& Filename : Filenames)
		{
			FStringView Extension = FPathViews::GetExtension(Filename, true);
			if (Extension != FPackageName::GetMapPackageExtension())
			{
				OutPackageNames.Emplace(*FPackageName::FilenameToLongPackageName(Filename));
			}
		}
	};

	FPackageNames PackageNames;
	PackageNames.Reserve(16);
	GetPackageNamesFromPath(Path, PackageNames);
	TCHAR PackageName[FName::StringBufferSize];

	for (const FName& Name : PackageNames)
	{
		Name.ToString(PackageName);
		UPackage* Package = FindPackage(NULL, PackageName);

		if (Package)
		{
			Package->FullyLoad();
		}
		else
		{
			Package = LoadPackage(NULL, PackageName, LOAD_None);
		}

		if (Package)
		{
			ForEachObjectWithOuter(Package, [Type, &OutAssets](UObject* Object)
			{
				const bool bWantedType = 
					((EAssetToLoad::ATL_Regular == Type) && Object->IsAsset()) ||
					((EAssetToLoad::ATL_Class == Type) && Object->IsA<UClass>());
				if (bWantedType)
				{
					OutAssets.Add(Object);
				}
			});
		}
	}
	return true;
}

TArray<FSubLevelStatus> GetSubLevelsStatus( UWorld* World, bool SortByActorCount )
{
	TArray<FSubLevelStatus> Result;
	FWorldContext &Context = GEngine->GetWorldContextFromWorldChecked(World);

	Result.Reserve(World->GetStreamingLevels().Num() + 1);
	
	// Add persistent level
	{
		FSubLevelStatus LevelStatus = {};
		LevelStatus.PackageName = World->GetOutermost()->GetFName();
		LevelStatus.StreamingStatus = LEVEL_Visible;
		LevelStatus.LODIndex = INDEX_NONE;
		LevelStatus.ActorCount = World->GetActorCount();

		if (const IWorldPartitionCell* WorldPartitionCell = World->PersistentLevel->GetWorldPartitionRuntimeCell())
		{
			LevelStatus.LevelLabel = WorldPartitionCell->GetDebugName();
		}

		Result.Add(LevelStatus);
	}

	auto SortFunc = [](ULevelStreaming* LevelA, ULevelStreaming* LevelB)
	{
		if (!LevelA->GetLoadedLevel())
		{
			return false;
		}
		else if (!LevelB->GetLoadedLevel())
		{
			return true;
		}

		return LevelA->GetLoadedLevel()->Actors.Num() > LevelB->GetLoadedLevel()->Actors.Num();
	};

	TArray<ULevelStreaming*> SortedStreamingLevels = World->GetStreamingLevels();

	if (SortByActorCount)
	{
		Algo::Sort(SortedStreamingLevels, SortFunc);
	}

	// Iterate over the world info's level streaming objects to find and see whether levels are loaded, visible or neither.
	for (const ULevelStreaming* LevelStreaming : SortedStreamingLevels)
	{
		if( LevelStreaming 
			&&  !LevelStreaming->GetWorldAsset().IsNull()
			&&	LevelStreaming->GetWorldAsset() != World )
		{
			FSubLevelStatus LevelStatus = {};
			LevelStatus.PackageName = LevelStreaming->GetWorldAssetPackageFName();
			LevelStatus.LODIndex = LevelStreaming->GetLevelLODIndex();
			LevelStatus.StreamingStatus = LevelStreaming->GetLevelStreamingStatus();

			if (LevelStreaming->GetLoadedLevel())
			{
				if (const IWorldPartitionCell* WorldPartitionCell = LevelStreaming->GetWorldPartitionCell())
				{
					LevelStatus.LevelLabel = WorldPartitionCell->GetDebugName();
				}

				LevelStatus.ActorCount = LevelStreaming->GetLoadedLevel()->Actors.Num();

				for (const AActor* Actor : LevelStreaming->GetLoadedLevel()->Actors)
				{
					if (Actor && !Actor->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
					{
						const UClass* ParentNativeClass = GetParentNativeClass(Actor->GetClass());
						FName NativeClassName = ParentNativeClass ? ParentNativeClass->GetFName() : NAME_None;

						FName ActorClassName = Actor->GetClass()->GetFName();
						FSubLevelActorDetails& ActorDetails = LevelStatus.ActorMapToCount.FindOrAdd(ActorClassName);
						ActorDetails.Count++;
						ActorDetails.NativeClassName = NativeClassName;
					}
				}
			}

			if (SortByActorCount)
			{
				LevelStatus.ActorMapToCount.ValueSort([](const FSubLevelActorDetails& A, const FSubLevelActorDetails& B) {
					return A.Count > B.Count;
				});
			}

			Result.Add(LevelStatus);
		}
	}

	
	// toss in the levels being loaded by PrepareMapChange
	for( int32 LevelIndex=0; LevelIndex < Context.LevelsToLoadForPendingMapChange.Num(); LevelIndex++ )
	{
		const FName LevelName = Context.LevelsToLoadForPendingMapChange[LevelIndex];
		
		FSubLevelStatus LevelStatus = {};
		LevelStatus.PackageName = LevelName;
		LevelStatus.StreamingStatus = LEVEL_Preloading;
		LevelStatus.LODIndex = INDEX_NONE;
		LevelStatus.ActorCount = 0;
		Result.Add(LevelStatus);
	}


	for( FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		if (APlayerController* PlayerController = Iterator->Get())
		{
			if (APawn* PCPawn = PlayerController->GetPawn())
			{
				// need to do a trace down here
				//TraceActor = Trace( out_HitLocation, out_HitNormal, TraceDest, TraceStart, false, TraceExtent, HitInfo, true );
				FHitResult Hit(1.f);

				// this will not work for flying around :-(
				PlayerController->GetWorld()->LineTraceSingleByObjectType(Hit, PCPawn->GetActorLocation(), (PCPawn->GetActorLocation() - FVector(0.f, 0.f, 256.f)), FCollisionObjectQueryParams(ECC_WorldStatic), FCollisionQueryParams(SCENE_QUERY_STAT(FindLevel), true, PCPawn));

				ULevel* LevelPlayerIsIn = nullptr;

				if (Hit.HitObjectHandle.IsValid())
				{
					LevelPlayerIsIn = Hit.HitObjectHandle.GetLevel();
				}
				else if (UPrimitiveComponent* HitComponent = Hit.Component.Get())
				{
					LevelPlayerIsIn = HitComponent->GetComponentLevel();
				}

				if (LevelPlayerIsIn)
				{
					FName LevelName = LevelPlayerIsIn->GetOutermost()->GetFName();
					FSubLevelStatus* LevelStatusPlayerIn = Result.FindByPredicate([LevelName](const FSubLevelStatus& InLevelStatus)
					{
						return InLevelStatus.PackageName == LevelName;
					});

					if (LevelStatusPlayerIn)
					{
						LevelStatusPlayerIn->bPlayerInside = true;
					}
				}
			}
		}
	}

	return Result;
}

//////////////////////////////////////////////////////////////////////////
// FConsoleOutputDevice

void FConsoleOutputDevice::Serialize(const TCHAR* Text, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	FStringOutputDevice::Serialize(Text, Verbosity, Category);
	FStringOutputDevice::Serialize(TEXT("\n"), Verbosity, Category);
	GLog->Serialize(Text, Verbosity, Category);

	if( Console != NULL )
	{
		bool bLogToConsole = true;

		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("con.MinLogVerbosity"));

		if(CVar)
		{
			int MinVerbosity = CVar->GetValueOnAnyThread(true);

			if((int)Verbosity <= MinVerbosity)
			{
				// in case the cvar is used we don't need this printout (avoid double print)
				bLogToConsole = false;
			}
		}

		if(bLogToConsole)
		{
			Console->OutputText(Text);
		}
	}
}

/*-----------------------------------------------------------------------------
	Serialized data stripping.
-----------------------------------------------------------------------------*/
FStripDataFlags::FStripDataFlags( class FArchive& Ar, uint8 InClassFlags /*= 0*/, const FPackageFileVersion& InVersion /*= GOldestLoadablePackageFileUEVersion */ )
{
	check(InVersion >= VER_UE4_OLDEST_LOADABLE_PACKAGE);
	if (Ar.UEVer().IsCompatible(InVersion))
	{
		if (Ar.IsCooking())
		{
			// When cooking GlobalStripFlags are automatically generated based on the current target
			// platform's properties.
			GlobalStripFlags |= Ar.CookingTarget()->HasEditorOnlyData() ? static_cast<uint8>(FStripDataFlags::EStrippedData::None) : static_cast<uint8>(FStripDataFlags::EStrippedData::EditorOnly);
			GlobalStripFlags |= Ar.CookingTarget()->AllowAudioVisualData() ? static_cast<uint8>(FStripDataFlags::EStrippedData::None) : static_cast<uint8>(FStripDataFlags::EStrippedData::AudioVisual);
			GlobalStripFlags |= Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::CanCookPackages) ? static_cast<uint8>(FStripDataFlags::EStrippedData::None) : static_cast<uint8>(FStripDataFlags::EStrippedData::NeededForCooking);
			ClassStripFlags = InClassFlags;
		}
		Ar << GlobalStripFlags;
		Ar << ClassStripFlags;
	}
}

FStripDataFlags::FStripDataFlags( class FArchive& Ar, uint8 InGlobalFlags, uint8 InClassFlags, const FPackageFileVersion& InVersion /*= GOldestLoadablePackageFileUEVersion */)
{
	check(InVersion >= VER_UE4_OLDEST_LOADABLE_PACKAGE);
	if (Ar.UEVer().IsCompatible(InVersion))
	{
		if (Ar.IsCooking())
		{
			// Don't generate global strip flags and use the ones passed in by the caller.
			GlobalStripFlags = InGlobalFlags;
			ClassStripFlags = InClassFlags;
		}
		Ar << GlobalStripFlags;
		Ar << ClassStripFlags;
	}
}

/*-----------------------------------------------------------------------------
Serialized data stripping.
-----------------------------------------------------------------------------*/
FStripDataFlags::FStripDataFlags(FStructuredArchive::FSlot Slot, uint8 InClassFlags /*= 0*/, const FPackageFileVersion& InVersion /*= GOldestLoadablePackageFileUEVersion */)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	check(InVersion >= VER_UE4_OLDEST_LOADABLE_PACKAGE);
	if (UnderlyingArchive.UEVer().IsCompatible(InVersion))
	{
		if (UnderlyingArchive.IsCooking())
		{
			// When cooking GlobalStripFlags are automatically generated based on the current target
			// platform's properties.
			GlobalStripFlags |= UnderlyingArchive.IsFilterEditorOnly() ? static_cast<uint8>(FStripDataFlags::EStrippedData::EditorOnly) : static_cast<uint8>(FStripDataFlags::EStrippedData::None);
			GlobalStripFlags |= !UnderlyingArchive.CookingTarget()->AllowAudioVisualData() ? static_cast<uint8>(FStripDataFlags::EStrippedData::AudioVisual) : static_cast<uint8>(FStripDataFlags::EStrippedData::None);
			GlobalStripFlags |= UnderlyingArchive.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::CanCookPackages) ? static_cast<uint8>(FStripDataFlags::EStrippedData::None) : static_cast<uint8>(FStripDataFlags::EStrippedData::NeededForCooking);
			ClassStripFlags = InClassFlags;
		}
		Record << SA_VALUE(TEXT("GlobalStripFlags"), GlobalStripFlags);
		Record << SA_VALUE(TEXT("ClassStripFlags"), ClassStripFlags);
	}
}

FStripDataFlags::FStripDataFlags(FStructuredArchive::FSlot Slot, uint8 InGlobalFlags, uint8 InClassFlags, const FPackageFileVersion& InVersion /*= GOldestLoadablePackageFileUEVersion */)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	check(InVersion >= VER_UE4_OLDEST_LOADABLE_PACKAGE);
	if (UnderlyingArchive.UEVer().IsCompatible(InVersion))
	{
		if (UnderlyingArchive.IsCooking())
		{
			// Don't generate global strip flags and use the ones passed in by the caller.
			GlobalStripFlags = InGlobalFlags;
			ClassStripFlags = InClassFlags;
		}
		Record << SA_VALUE(TEXT("GlobalStripFlags"), GlobalStripFlags);
		Record << SA_VALUE(TEXT("ClassStripFlags"), ClassStripFlags);
	}
}


void VirtualTextureUtils::CheckAndReportInvalidUsage(const UObject* Owner, const FName& PropertyName, const UTexture* Texture)
{
	if (Texture && Texture->VirtualTextureStreaming)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TextureName"), FText::FromName(Texture->GetFName()));
		Arguments.Add(TEXT("ObjectName"), FText::FromName(Owner->GetFName()));
		Arguments.Add(TEXT("PropertyName"), FText::FromName(PropertyName));
		auto Log = FMessageLog("MapCheck");
		Log.Warning()
			->AddToken(FUObjectToken::Create(Owner))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_InvalidVirtualTextureUsage", "{ObjectName} is using a virtual texture ('{TextureName}') on an unsupported property ('{PropertyName}')."), Arguments)))
			->AddToken(FMapErrorToken::Create(FMapErrors::InvalidVirtualTextureUsage));
		Log.Open();
	}
}

#undef LOCTEXT_NAMESPACE
