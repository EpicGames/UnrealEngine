// Copyright Epic Games, Inc. All Rights Reserved.

/*===========================================================================
	ErrorChecking.cpp
	Actor Error checking functions
  ===========================================================================*/

#include "UObject/Package.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "GameFramework/PainCausingVolume.h"
#include "Engine/LevelStreamingVolume.h"
#include "Engine/Light.h"
#include "Engine/Note.h"
#include "ActorEditorUtils.h"
#include "Engine/World.h"
#include "Model.h"
#include "Engine/Polys.h"
#include "Engine/LevelStreaming.h"

#define LOCTEXT_NAMESPACE "ErrorChecking"

/**
 * Special archive for finding references from a map to objects contained within an editor-only package.
 */
class FEditorContentReferencersArchive : public FArchiveUObject
{
public:
	FEditorContentReferencersArchive( const TArray<UPackage*>& inEditorContentPackages )
	: EditorContentPackages(inEditorContentPackages)
	{
		ArIsObjectReferenceCollector = true;
		this->SetIsPersistent(true);
		ArIgnoreClassRef = true;
	}

	/** 
	 * UObject serialize operator implementation
	 *
	 * @param Object	reference to Object reference
	 * @return reference to instance of this class
	 */
	FArchive& operator<<( UObject*& Object )
	{
		// Avoid duplicate entries.
		if ( Object != NULL && !SerializedObjects.Contains(Object) )
		{
			SerializedObjects.Add(Object);
			if ( !Object->IsA(UField::StaticClass()) 
			&&	(Object->NeedsLoadForClient() || Object->NeedsLoadForServer()) )
			{
				if (EditorContentPackages.Contains(Object->GetOutermost())
				&&	Object->GetOutermost() != Object )
				{
					ReferencedEditorOnlyObjects.Add(Object);
				}

				Object->Serialize(*this);
			}
		}
		
		return *this;
	}

	/** the list of objects within the editor-only packages that are referenced by this map */
	TSet<UObject*> ReferencedEditorOnlyObjects;

protected:

	/** the list of packages that will not be loaded in-game */
	TArray<UPackage*> EditorContentPackages;

	/** prevents objects from being serialized more than once */
	TSet<UObject*> SerializedObjects;
};

#if WITH_EDITOR


void APainCausingVolume::CheckForErrors()
{
	Super::CheckForErrors();

	if ( !DamageType )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_NoDamageType", "{ActorName} : Causes damage but has no DamageType defined" ), Arguments) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::NoDamageType));
	}
}

void ANote::CheckForErrors()
{
	Super::CheckForErrors();

	if( Text.Len() > 0 )
	{
		FMessageLog("MapCheck").Info()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create( FText::FromString( Text ) ) );
	}
}

void ABrush::CheckForErrors()
{
	Super::CheckForErrors();

	if( !BrushComponent )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_BrushComponentNull", "{ActorName} : Brush has NULL BrushComponent property - please delete" ), Arguments) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::BrushComponentNull));
	}
	else
	{		
		// NOTE : don't report NULL texture references on the builder brush - it doesn't matter there
		if( Brush && !FActorEditorUtils::IsABuilderBrush(this) && !IsBrushShape() )
		{
			if ( Brush->Polys )
			{
				// A brush without any polygons in it isn't useful.  Should be deleted.
				if( Brush->Polys->Element.Num() == 0 )
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
					FMessageLog("MapCheck").Warning()
						->AddToken(FUObjectToken::Create(this))
						->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_BrushZeroPolygons", "{ActorName} : Brush has zero polygons - please delete" ), Arguments) ))
						->AddToken(FMapErrorToken::Create(FMapErrors::BrushZeroPolygons));
				}

				// Check for non-coplanar polygons.
				for( int32 x = 0 ; x < Brush->Polys->Element.Num() ; ++x )
				{
					FPoly* Poly = &(Brush->Polys->Element[x]);
					bool	Coplanar = 1;

					for(int32 VertexIndex = 0;VertexIndex < Poly->Vertices.Num();VertexIndex++)
					{
						if(FMath::Abs(FPlane((FVector)Poly->Vertices[0], (FVector)Poly->Normal).PlaneDot((FVector)Poly->Vertices[VertexIndex])) > UE_THRESH_POINT_ON_PLANE)
						{
							FFormatNamedArguments Arguments;
							Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
							FMessageLog("MapCheck").Warning()
								->AddToken(FUObjectToken::Create(this))
								->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_NonCoPlanarPolys", "{ActorName} : Brush has non-coplanar polygons" ), Arguments) ))
								->AddToken(FMapErrorToken::Create(FMapErrors::NonCoPlanarPolys));
							Coplanar = 0;
							break;
						}
					}

					if(!Coplanar)
					{
						break;
					}
				}
			}
			else
			{
				// A brush without any polygons in it is invalid.  Should be deleted.
				FFormatNamedArguments Arguments;
                Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
                FMessageLog("MapCheck").Warning()
                	->AddToken(FUObjectToken::Create(this))
                	->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_BrushNullPolygons", "{ActorName} : Brush has a NULL polygon reference - please delete" ), Arguments) ))
                	->AddToken(FMapErrorToken::Create(FMapErrors::BrushZeroPolygons));
			}
			

			// check for planar brushes which might mess up collision
			if(Brush->Bounds.BoxExtent.Z < UE_SMALL_NUMBER || Brush->Bounds.BoxExtent.Y < UE_SMALL_NUMBER || Brush->Bounds.BoxExtent.X < UE_SMALL_NUMBER)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_PlanarBrush", "{ActorName} : Brush is planar" ), Arguments) ))
					->AddToken(FMapErrorToken::Create(FMapErrors::PlanarBrush));
			}
		}
	}
}

/**
 * Function that gets called from within Map_Check to allow this actor to check itself
 * for any potential errors and register them with map check dialog.
 */
void AVolume::CheckForErrors()
{
	Super::CheckForErrors();

	// Some volumes do not need a valid collision component; for example:
	// the default physics volume only uses the physics properties and not the extents - so it will have an area of zero
	if (ShouldCheckCollisionComponentForErrors())
	{
		if (GetRootComponent() == NULL)
		{
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_VolumeActorCollisionComponentNULL", "Volume actor has NULL collision component - please delete")))
				->AddToken(FMapErrorToken::Create(FMapErrors::VolumeActorCollisionComponentNULL));
		}
		else
		{
			if (GetRootComponent()->Bounds.SphereRadius <= UE_SMALL_NUMBER)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_VolumeActorZeroRadius", "Volume actor has a collision component with 0 radius - please delete")));
			}
		}
	}
}


void ALight::CheckForErrors()
{
	Super::CheckForErrors();
	if( !LightComponent )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_LightComponentNull", "Light actor has NULL LightComponent property - please delete!")))
			->AddToken(FMapErrorToken::Create(FMapErrors::LightComponentNull));
	}
}

void ALevelStreamingVolume::CheckForErrors()
{
	Super::CheckForErrors();

	// Streaming level volumes are not permitted outside the persistent level.
	if ( GetLevel() != GetWorld()->PersistentLevel )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_LevelStreamingVolume", "{ActorName} : LevelStreamingVolume is not in the persistent level - please delete" ), Arguments) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::LevelStreamingVolume));
	}

	// Warn if the volume has no streaming levels associated with it
	bool bHasAssociatedLevels = false;
	for (ULevelStreaming* LevelStreaming : GetWorld()->GetStreamingLevels())
	{
		if (LevelStreaming && LevelStreaming->EditorStreamingVolumes.Contains(this))
		{
			bHasAssociatedLevels = true;
			break;
		}
	}

	if ( !bHasAssociatedLevels )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_NoLevelsAssociatedWithStreamingVolume", "{ActorName} : No levels are associated with streaming volume." ), Arguments) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::NoLevelsAssociated));
	}
}
#endif

#undef LOCTEXT_NAMESPACE
