// Copyright Epic Games, Inc. All Rights Reserved.
#include "Engine/Experimental/StreamableManagerError.h"
#include "UObject/UObjectGlobals.h"

UE_DEFINE_ERROR_MODULE(StreamableManager);

UE_DEFINE_ERROR(StreamableManager, PackageLoadFailed);
UE_DEFINE_ERROR(StreamableManager, PackageLoadCanceled);
UE_DEFINE_ERROR(StreamableManager, DownloadError);
UE_DEFINE_ERROR(StreamableManager, PackageNameInvalid);
UE_DEFINE_ERROR(StreamableManager, IoStoreNotFound);
UE_DEFINE_ERROR(StreamableManager, SyncLoadIncomplete);
UE_DEFINE_ERROR(StreamableManager, AsyncLoadFailed);
UE_DEFINE_ERROR(StreamableManager, AsyncLoadCancelled);
UE_DEFINE_ERROR(StreamableManager, AsyncLoadUnknownError);
UE_DEFINE_ERROR(StreamableManager, UnknownError);
UE_DEFINE_ERROR(StreamableManager, AsyncLoadNotInstalled);

UE_DEFINE_ERROR_CONTEXT(StreamableManager, AdditionalContext)
UE_DEFINE_ERROR_CONTEXT(StreamableManager, RequestContext);

namespace StreamableManager
{

UE::UnifiedError::FError GetStreamableError(EAsyncLoadingResult::Type Result)
{
	switch (Result)
	{
	case EAsyncLoadingResult::Failed:
	case EAsyncLoadingResult::FailedMissing:
	case EAsyncLoadingResult::FailedLinker:
		// We could supply an error string if async loading bubbled one up...
		// Possibly GetExplanationForUnavailablePackage?
		return ::StreamableManager::AsyncLoadFailed();
	case EAsyncLoadingResult::FailedNotInstalled:
		return ::StreamableManager::AsyncLoadNotInstalled();
	case EAsyncLoadingResult::Canceled:
		return ::StreamableManager::AsyncLoadCancelled();
	}

	return ::StreamableManager::AsyncLoadUnknownError((int32)Result);
}

} // namespace StreamableManager