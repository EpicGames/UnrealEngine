// Copyright Epic Games, Inc. All Rights Reserved.
#include "ExternalActorAssetDependencyGatherer.h"

#if WITH_EDITOR

#include "AssetRegistry/ARFilter.h"
#include "Engine/World.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Engine/Level.h"

void FExternalActorAssetDependencyGatherer::GatherDependencies(FGatherDependenciesContext& Context) const
{
	if (ULevel::GetIsLevelUsingExternalActorsFromAsset(Context.GetAssetData()))
	{
		const FString ExternalActorsPath = ULevel::GetExternalActorsPath(Context.GetAssetData().PackageName.ToString());
		Context.GetOutDependencyDirectories().Add(ExternalActorsPath);

		FARFilter Filter;
		Filter.bRecursivePaths = true;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.PackagePaths.Add(*ExternalActorsPath);

		TArray<FAssetData> FilteredAssets;
		Context.GetAssetRegistryState().GetAssets(Context.CompileFilter(Filter), {}, FilteredAssets, true);

		for (const FAssetData& FilteredAsset : FilteredAssets)
		{
			Context.GetOutDependencies().Emplace(IAssetDependencyGatherer::FGathereredDependency{ FilteredAsset.PackageName,
				UE::AssetRegistry::EDependencyProperty::Game  | UE::AssetRegistry::EDependencyProperty::Build });
		}
	}
}

#endif
