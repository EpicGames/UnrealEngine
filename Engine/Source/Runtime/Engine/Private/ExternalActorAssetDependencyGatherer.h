// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "AssetRegistry/AssetDependencyGatherer.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/World.h"

class FExternalActorAssetDependencyGatherer : public IAssetDependencyGatherer
{
public:
	
	FExternalActorAssetDependencyGatherer() = default;
	virtual ~FExternalActorAssetDependencyGatherer() = default;

	virtual void GatherDependencies(FGatherDependenciesContext& Context) const override;
};

#endif