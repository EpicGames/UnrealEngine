// Copyright Epic Games, Inc. All Rights Reserved.
#include "Engine/ExternalAssetDependencyGatherer.h"

#if WITH_EDITOR

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "ExternalPackageHelper.h"

void FExternalAssetDependencyGatherer::GatherDependencies(FGatherDependenciesContext& Context) const
{		
	const FString ExternalObjectsPath = FExternalPackageHelper::GetExternalObjectsPath(Context.GetAssetData().PackageName.ToString());
	Context.GetOutDependencyDirectories().Add(ExternalObjectsPath);

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = true;
	Filter.PackagePaths.Add(*ExternalObjectsPath);

	TArray<FAssetData> FilteredAssets;
	Context.GetAssetRegistryState().GetAssets(Context.CompileFilter(Filter), {}, FilteredAssets, true);

	for (const FAssetData& FilteredAsset : FilteredAssets)
	{
		Context.GetOutDependencies().Emplace(IAssetDependencyGatherer::FGathereredDependency{ FilteredAsset.PackageName, UE::AssetRegistry::EDependencyProperty::Game | UE::AssetRegistry::EDependencyProperty::Build });
	}
}

#endif
