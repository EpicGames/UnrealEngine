// Copyright Epic Games, Inc. All Rights Reserved.
#include "Engine/ExternalObjectAndActorDependencyGatherer.h"

#if WITH_EDITOR

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "ExternalPackageHelper.h"

void FExternalObjectAndActorDependencyGatherer::GatherDependencies(FGatherDependenciesContext& Params) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FExternalObjectAndActorDependencyGatherer::GatherDependencies);

	if (ExternalPathsProvider)
	{
		IExternalAssetPathsProvider::FUpdateCacheContext UpdateContext{Params.GetAssetRegistryState(), Params.GetCachedPathTree(), Params.GetCompileFilterFunc() };
		ExternalPathsProvider->UpdateCache(UpdateContext);
	}

	FARFilter Filter = GetQueryFilter(Params.GetAssetData().PackageName, &Params.GetOutDependencyDirectories());
		
	TArray<FAssetData> FilteredAssets;
	Params.GetAssetRegistryState().GetAssets(Params.CompileFilter(Filter), {}, FilteredAssets, true);

	Params.GetOutDependencies().Reserve(FilteredAssets.Num());
	for (const FAssetData& FilteredAsset : FilteredAssets)
	{
		Params.GetOutDependencies().Emplace(IAssetDependencyGatherer::FGathereredDependency{ FilteredAsset.PackageName, UE::AssetRegistry::EDependencyProperty::Game | UE::AssetRegistry::EDependencyProperty::Build });
	}
}

FARFilter FExternalObjectAndActorDependencyGatherer::GetQueryFilter(FName PackageName, TArray<FString>* OutQueryDirectories)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FExternalObjectAndActorDependencyGatherer::GetQueryFilter);

	const FString ExternalActorsPath = ULevel::GetExternalActorsPath(PackageName.ToString());
	const FString ExternalObjectPath = FExternalPackageHelper::GetExternalObjectsPath(PackageName.ToString());
	if (OutQueryDirectories)
	{
		OutQueryDirectories->Add(ExternalActorsPath);
		OutQueryDirectories->Add(ExternalObjectPath);
	}

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = true;
	Filter.PackagePaths.Add(FName(ExternalActorsPath));
	Filter.PackagePaths.Add(FName(ExternalObjectPath));

	if (ExternalPathsProvider && IsRunningCookCommandlet())
	{
		TArray<FName> ExternalPaths = ExternalPathsProvider->GetPathsForPackage(PackageName);
		for (FName ExternalPath : ExternalPaths)
		{
			Filter.PackagePaths.Add(ExternalPath);
			if (OutQueryDirectories)
			{
				OutQueryDirectories->Add(ExternalPath.ToString());
			}
		}
	}
		
	return Filter;
}

void FExternalObjectAndActorDependencyGatherer::SetExternalAssetPathsProvider(IExternalAssetPathsProvider* InProvider)
{
	// Can't be set over an already set provider, or we make this a collection
	check(!InProvider || !ExternalPathsProvider);
	
	ExternalPathsProvider = InProvider;
}

IExternalAssetPathsProvider* FExternalObjectAndActorDependencyGatherer::ExternalPathsProvider = nullptr;

REGISTER_ASSETDEPENDENCY_GATHERER(FExternalObjectAndActorDependencyGatherer, UWorld);

#endif

