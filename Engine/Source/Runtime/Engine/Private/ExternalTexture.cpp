// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalTexture.h"
#include "MaterialShared.h"
#include "Materials/MaterialRenderProxy.h"


#define EXTERNALTEXTURE_TRACE_REGISTRY 0


FExternalTextureRegistry* FExternalTextureRegistry::Singleton = nullptr;


FExternalTextureRegistry& FExternalTextureRegistry::Get()
{
	check(IsInParallelRenderingThread());

	if (Singleton == nullptr)
	{
		Singleton = new FExternalTextureRegistry();
	}

	return *Singleton;
}


void FExternalTextureRegistry::RegisterExternalTexture(const FGuid& InGuid, FTextureRHIRef& InTextureRHI, FSamplerStateRHIRef& InSamplerStateRHI, const FLinearColor& InCoordinateScaleRotation, const FLinearColor& InCoordinateOffset)
{
	check(IsInRenderingThread());
	FScopeLock Lock(&CriticalSection);
	TextureEntries.Add(InGuid, FExternalTextureEntry(InTextureRHI, InSamplerStateRHI, InCoordinateScaleRotation, InCoordinateOffset));

	for (const FMaterialRenderProxy* MaterialRenderProxy : ReferencingMaterialRenderProxies)
	{
		const_cast<FMaterialRenderProxy*>(MaterialRenderProxy)->CacheUniformExpressions(FRHICommandListImmediate::Get(), false);
	}
}


void FExternalTextureRegistry::UnregisterExternalTexture(const FGuid& InGuid)
{
	check(IsInRenderingThread());
	FScopeLock Lock(&CriticalSection);
	TextureEntries.Remove(InGuid);

	for (const FMaterialRenderProxy* MaterialRenderProxy : ReferencingMaterialRenderProxies)
	{
		const_cast<FMaterialRenderProxy*>(MaterialRenderProxy)->CacheUniformExpressions(FRHICommandListImmediate::Get(), false);
	}
}


void FExternalTextureRegistry::RemoveMaterialRenderProxyReference(const FMaterialRenderProxy* MaterialRenderProxy)
{
	FScopeLock Lock(&CriticalSection);
	ReferencingMaterialRenderProxies.Remove(MaterialRenderProxy);
}


bool FExternalTextureRegistry::GetExternalTexture(const FMaterialRenderProxy* MaterialRenderProxy, const FGuid& InGuid, FTextureRHIRef& OutTextureRHI, FSamplerStateRHIRef& OutSamplerStateRHI)
{
#if EXTERNALTEXTURE_TRACE_REGISTRY
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("GetExternalTexture: Guid = %s"), *InGuid.ToString());
#endif
	FScopeLock Lock(&CriticalSection);

	if ((MaterialRenderProxy != nullptr) && MaterialRenderProxy->IsMarkedForGarbageCollection())
	{
		UE_LOGF(LogMaterial, Fatal, "FMaterialRenderProxy was already marked for garbage collection");
	}

	// register material proxy if already initialized
	if ((MaterialRenderProxy != nullptr) && MaterialRenderProxy->IsInitialized())
	{
		ReferencingMaterialRenderProxies.Add(MaterialRenderProxy);

		// Note: FMaterialRenderProxy::ReleaseRHI()
		// is responsible for removing the material proxy
	}

	if (!InGuid.IsValid())
	{
		return false; // no identifier associated with the texture yet
	}

	FExternalTextureEntry* Entry = TextureEntries.Find(InGuid);

	if (Entry == nullptr)
	{
#if EXTERNALTEXTURE_TRACE_REGISTRY
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("GetExternalTexture: NOT FOUND!"));
#endif

		return false; // texture not registered
	}

#if EXTERNALTEXTURE_TRACE_REGISTRY
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("GetExternalTexture: Found"));
#endif

	OutTextureRHI = Entry->TextureRHI;
	OutSamplerStateRHI = Entry->SamplerStateRHI;

	return true;
}


bool FExternalTextureRegistry::GetExternalTextureCoordinateScaleRotation(const FGuid& InGuid, FLinearColor& OutCoordinateScaleRotation)
{
	FScopeLock Lock(&CriticalSection);

	FExternalTextureEntry* Entry = TextureEntries.Find(InGuid);

	if (Entry == nullptr)
	{
		return false;
	}

	OutCoordinateScaleRotation = Entry->CoordinateScaleRotation;

	return true;
}


bool FExternalTextureRegistry::GetExternalTextureCoordinateOffset(const FGuid& InGuid, FLinearColor& OutCoordinateOffset)
{
	FScopeLock Lock(&CriticalSection);

	FExternalTextureEntry* Entry = TextureEntries.Find(InGuid);

	if (Entry == nullptr)
	{
		return false;
	}

	OutCoordinateOffset = Entry->CoordinateOffset;

	return true;
}
