// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotification/FieldNotificationLibrary.h"

#include "INotifyFieldValueChanged.h"
#include "GameFramework/Actor.h"
#include "Net/NetPushModelHelpers.h"
#include "UObject/Script.h"
#include "Blueprint/BlueprintExceptionInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FieldNotificationLibrary)

#define LOCTEXT_NAMESPACE "FieldNotificationLibrary"

namespace UE::FieldNotification::Private
{
bool SetPropertyValueAndBroadcast(void* SourceValuePtr, void* TargetValuePtr, FProperty* TargetProperty, UObject* Self
	, UObject* NetOwner, bool bHasLocalRepNotify, bool bShouldFlushDormancyOnSet, bool bIsNetProperty, TArrayView<FFieldNotificationId> FieldIds
	, const UObject* ActiveObject, FFrame& StackFrame)
{
	TScriptInterface<INotifyFieldValueChanged> NotifyFieldSelf = Self;
	if (NotifyFieldSelf.GetObject() == nullptr || NotifyFieldSelf.GetInterface() == nullptr || TargetProperty == nullptr)
	{
		const FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("MissingInputProperty", "Failed to resolve the input parameter for SetPropertyValueAndBroadcast.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(ActiveObject, StackFrame, ExceptionInfo);
		return false;
	}

	check(SourceValuePtr);
	check(TargetValuePtr);
	check(Self);

	//if (Old!=New) then
	//	FlushNetDormancy
	//	SetValue
	//	MarkPropertyDirty
	//	OnRep (it include BroadcastFieldValueChanged) or BroadcastFieldValueChanged

	bool bResult = !TargetProperty->Identical(TargetValuePtr, SourceValuePtr);
	if (bResult)
	{
		// FlushNetDormancy
		if (bShouldFlushDormancyOnSet)
		{
			// if it has an actor, Call FlushNetDormancy
			if (AActor* ReplicationOwner = Cast<AActor>(NetOwner))
			{
				ReplicationOwner->FlushNetDormancy();
			}
		}

		// SetValue
		TargetProperty->SetValue_InContainer(Self, SourceValuePtr);

		bool bCallBroadcastFieldValueChanged = true;
		if (bIsNetProperty)
		{
			// MarkPropertyDirty
			if (TargetProperty->RepIndex != INDEX_NONE)
			{
				UNetPushModelHelpers::MarkPropertyDirtyFromRepIndex(Self, TargetProperty->RepIndex, TargetProperty->GetFName());
			}

			// OnRep It will do the BroadcastFieldValueChanged inside the OnRep function because we added it at compile time.
			if (bHasLocalRepNotify && !TargetProperty->RepNotifyFunc.IsNone())
			{
				if (UFunction* Function = Self->GetClass()->FindFunctionByName(TargetProperty->RepNotifyFunc))
				{
					if (Function->NumParms == 0 && Function->GetReturnProperty() == nullptr)
					{
						Self->ProcessEvent(Function, nullptr);
						bCallBroadcastFieldValueChanged = false;
					}
				}
			}
		}

		// BroadcastFieldValueChanged
		if (bCallBroadcastFieldValueChanged)
		{
			for (FFieldNotificationId FieldNotificationId : FieldIds)
			{
				UE::FieldNotification::FFieldId FieldId = NotifyFieldSelf->GetFieldNotificationDescriptor().GetField(Self->GetClass(), FieldNotificationId.GetFieldName());
				if (ensureMsgf(FieldId.IsValid(),
					TEXT("BroadcastFieldValueChanged: Field '%s' could not be resolved in the FieldNotificationDescriptor for class '%s' (object: '%s') "
						 "while setting TargetProperty '%s'. "
						 "This usually means the field was removed or renamed after the Blueprint node was compiled. "
						 "Re-open and recompile the Blueprint that calls this node, or verify the field still exists on the class."),
					*FieldNotificationId.GetFieldName().ToString(),
					*Self->GetClass()->GetPathName(),
					*Self->GetName(),
					*TargetProperty->GetPathName()))
				{
					NotifyFieldSelf->BroadcastFieldValueChanged(FieldId);
				}
			}
		}
	}

	return bResult;
}
}

void UFieldNotificationLibrary::BroadcastFieldValueChanged(UObject* Object, FFieldNotificationId InFieldNotificationId)
{
	TScriptInterface<INotifyFieldValueChanged> NotifyFieldSelf = Object;
	if (NotifyFieldSelf.GetObject() != nullptr && NotifyFieldSelf.GetInterface() != nullptr && InFieldNotificationId.IsValid())
	{
		const UE::FieldNotification::FFieldId FieldId = NotifyFieldSelf->GetFieldNotificationDescriptor().GetField(Object->GetClass(), InFieldNotificationId.GetFieldName());
		if (FieldId.IsValid())
		{
			NotifyFieldSelf->BroadcastFieldValueChanged(FieldId);
		}
	}
}

void UFieldNotificationLibrary::BroadcastFieldsValueChanged(UObject* Object, TArray<FFieldNotificationId> InFieldNotificationIds)
{
	TScriptInterface<INotifyFieldValueChanged> NotifyFieldSelf = Object;
	if (NotifyFieldSelf.GetObject() != nullptr && NotifyFieldSelf.GetInterface() != nullptr)
	{
		for (FFieldNotificationId FieldNotificationId : InFieldNotificationIds)
		{
			if (FieldNotificationId.IsValid())
			{
				const UE::FieldNotification::FFieldId FieldId = NotifyFieldSelf->GetFieldNotificationDescriptor().GetField(Object->GetClass(), FieldNotificationId.GetFieldName());
				if (FieldId.IsValid())
				{
					NotifyFieldSelf->BroadcastFieldValueChanged(FieldId);
				}
			}
		}
	}
}

DEFINE_FUNCTION(UFieldNotificationLibrary::execSetPropertyValueAndBroadcast)
{
	P_GET_UBOOL(NewValueByRef);

	// Old Value
	Stack.StepCompiledIn<FProperty>(nullptr);
	FProperty* TargetProperty = Stack.MostRecentProperty;
	void* TargetValuePtr = Stack.MostRecentPropertyAddress;

	// New Value
	void* SourceValuePtr = nullptr;
	if (NewValueByRef)
	{
		// Is linked in the graph
		Stack.StepCompiledIn<FProperty>(nullptr);
		SourceValuePtr = Stack.MostRecentPropertyAddress;
	}
	else if (TargetProperty)
	{
		// Is a default in the graph.
		//New Value type is the same as Old Value

		// Allocate temporary memory for the variable
		SourceValuePtr = FMemory_Alloca_Aligned(TargetProperty->GetSize(), TargetProperty->GetMinAlignment());
		TargetProperty->InitializeValue(SourceValuePtr);
		Stack.Step(Stack.Object, SourceValuePtr);
	}
	else
	{
		// Can't read the previous value, it won't be able to set it. Fails and code skip the context op code.
		Stack.bArrayContextFailed = true;
		return;
	}

	P_GET_OBJECT(UObject, Self);
	P_GET_OBJECT(UObject, NetOwner);
	P_GET_UBOOL(bHasLocalRepNotify);
	P_GET_UBOOL(bShouldFlushDormancyOnSet);
	P_GET_UBOOL(bIsNetProperty);

	P_FINISH;

	*(bool*)RESULT_PARAM = false;
	if (Self == nullptr || SourceValuePtr == nullptr)
	{
		const FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("MissingInputProperty", "Failed to resolve the input parameter for SetPropertyValueAndBroadcast.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if (TargetValuePtr)
	{
		P_NATIVE_BEGIN
		FFieldNotificationId FieldNotify = FFieldNotificationId(TargetProperty->GetFName());
		bool bResult = UE::FieldNotification::Private::SetPropertyValueAndBroadcast(SourceValuePtr, TargetValuePtr, TargetProperty, Self
			, NetOwner, bHasLocalRepNotify, bShouldFlushDormancyOnSet, bIsNetProperty, TArrayView<FFieldNotificationId>(&FieldNotify, 1), P_THIS, Stack);
		*(bool*)RESULT_PARAM = bResult;
		P_NATIVE_END
	}

	if (!NewValueByRef && SourceValuePtr)
	{
		TargetProperty->DestroyValue(SourceValuePtr);
	}
}

DEFINE_FUNCTION(UFieldNotificationLibrary::execSetPropertyValueAndBroadcastFields)
{
	P_GET_UBOOL(NewValueByRef);

	Stack.StepCompiledIn<FProperty>(nullptr);
	FProperty* TargetProperty = Stack.MostRecentProperty;
	void* TargetValuePtr = Stack.MostRecentPropertyAddress;

	void* SourceValuePtr = nullptr;
	if (NewValueByRef)
	{
		Stack.StepCompiledIn<FProperty>(nullptr);
		SourceValuePtr = Stack.MostRecentPropertyAddress;
	}
	else if (TargetProperty)
	{
		// Allocate temporary memory for the variable
		SourceValuePtr = FMemory_Alloca_Aligned(TargetProperty->GetSize(), TargetProperty->GetMinAlignment());
		TargetProperty->InitializeValue(SourceValuePtr); // probably not needed since they are double/float
		Stack.Step(Stack.Object, SourceValuePtr);
	}
	else
	{
		// Can't read the previous value, it won't be able to set it. Fails and code skip the context op code.
		Stack.bArrayContextFailed = true;
		return;
	}

	P_GET_OBJECT(UObject, Self);
	P_GET_OBJECT(UObject, NetOwner);
	P_GET_UBOOL(bHasLocalRepNotify);
	P_GET_UBOOL(bShouldFlushDormancyOnSet);
	P_GET_UBOOL(bIsNetProperty);
	P_GET_TARRAY(FFieldNotificationId, FieldNotifies);

	P_FINISH;

	*(bool*)RESULT_PARAM = false;
	if (Self == nullptr || SourceValuePtr == nullptr)
	{
		const FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("MissingInputProperty", "Failed to resolve the input parameter for SetPropertyValueAndBroadcast.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		
	}
	else if (SourceValuePtr && TargetValuePtr)
	{
		P_NATIVE_BEGIN
		FieldNotifies.AddUnique(FFieldNotificationId(TargetProperty->GetFName()));
		bool bResult = UE::FieldNotification::Private::SetPropertyValueAndBroadcast(SourceValuePtr, TargetValuePtr, TargetProperty, Self
			, NetOwner, bHasLocalRepNotify, bShouldFlushDormancyOnSet, bIsNetProperty, FieldNotifies, P_THIS, Stack);
		*(bool*)RESULT_PARAM = bResult;
		P_NATIVE_END
	}

	if (!NewValueByRef && SourceValuePtr)
	{
		TargetProperty->DestroyValue(SourceValuePtr);
	}
}

#undef LOCTEXT_NAMESPACE
