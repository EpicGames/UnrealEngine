// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/FloatingPawnMovement.h"
#include "Engine/HitResult.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "GameFramework/WorldSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatingPawnMovement)

UFloatingPawnMovement::UFloatingPawnMovement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MaxSpeed = 1200.f;
	Acceleration = 4000.f;
	Deceleration = 8000.f;
	TurningBoost = 8.0f;
	bPositionCorrected = false;

	ResetMoveState();
}

void UFloatingPawnMovement::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	if (ShouldSkipUpdate(DeltaTime))
	{
		return;
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (!PawnOwner || !UpdatedComponent)
	{
		return;
	}

	const AController* Controller = PawnOwner->GetController();
	if (Controller && Controller->IsLocalController())
	{
		// apply input for local players but also for AI that's not following a navigation path at the moment
		if (Controller->IsLocalPlayerController() == true || Controller->IsFollowingAPath() == false || NavMovementProperties.bUseAccelerationForPaths)
		{
			ApplyControlInputToVelocity(DeltaTime);
		}
		// if it's not player controller, but we do have a controller, then it's AI
		// (that's not following a path) and we need to limit the speed
		else if (IsExceedingMaxSpeed(MaxSpeed) == true)
		{
			Velocity = Velocity.GetUnsafeNormal() * MaxSpeed;
		}

		LimitWorldBounds();
		bPositionCorrected = false;

		// Move actor
		FVector Delta = Velocity * DeltaTime;

		if (!Delta.IsNearlyZero(1e-6f))
		{
			const FVector OldLocation = UpdatedComponent->GetComponentLocation();
			const FQuat Rotation = UpdatedComponent->GetComponentQuat();

			FHitResult Hit(1.f);
			SafeMoveUpdatedComponent(Delta, Rotation, true, Hit);

			if (Hit.IsValidBlockingHit())
			{
				HandleImpact(Hit, DeltaTime, Delta);
				// Try to slide the remaining distance along the surface.
				SlideAlongSurface(Delta, 1.f-Hit.Time, Hit.Normal, Hit, true);
			}

			// Update velocity
			// We don't want position changes to vastly reverse our direction (which can happen due to penetration fixups etc)
			if (!bPositionCorrected)
			{
				const FVector NewLocation = UpdatedComponent->GetComponentLocation();
				Velocity = ((NewLocation - OldLocation) / DeltaTime);
			}
		}
		
		// Finalize
		UpdateComponentVelocity();
	}
};

bool UFloatingPawnMovement::LimitWorldBounds()
{
	AWorldSettings* WorldSettings = PawnOwner ? PawnOwner->GetWorldSettings() : NULL;
	if (!WorldSettings || !WorldSettings->AreWorldBoundsChecksEnabled() || !UpdatedComponent)
	{
		return false;
	}

	const FVector CurrentLocation = UpdatedComponent->GetComponentLocation();
	if ( CurrentLocation.Z < WorldSettings->KillZ )
	{
		Velocity.Z = FMath::Min<FVector::FReal>(GetMaxSpeed(), WorldSettings->KillZ - CurrentLocation.Z + 2.0f);
		return true;
	}

	return false;
}

void UFloatingPawnMovement::ApplyControlInputToVelocity(float DeltaTime)
{
	const FVector ControlAcceleration = GetPendingInputVector().GetClampedToMaxSize(1.f);

	const float AnalogInputModifier = (ControlAcceleration.SizeSquared() > 0.f ? ControlAcceleration.Size() : 0.f);
	const float MaxPawnSpeed = GetMaxSpeed() * AnalogInputModifier;
	const bool bExceedingMaxSpeed = IsExceedingMaxSpeed(MaxPawnSpeed);

	if (AnalogInputModifier > 0.f && !bExceedingMaxSpeed)
	{
		// Apply change in velocity direction
		if (Velocity.SizeSquared() > 0.f)
		{
			// Change direction faster than only using acceleration, but never increase velocity magnitude.
			const float TimeScale = FMath::Clamp(DeltaTime * TurningBoost, 0.f, 1.f);
			Velocity = Velocity + (ControlAcceleration * Velocity.Size() - Velocity) * TimeScale;
		}
	}
	else
	{
		// Dampen velocity magnitude based on deceleration.
		if (Velocity.SizeSquared() > 0.f)
		{
			const FVector OldVelocity = Velocity;
			const float VelSize = FMath::Max(Velocity.Size() - FMath::Abs(Deceleration) * DeltaTime, 0.f);
			Velocity = Velocity.GetSafeNormal() * VelSize;

			// Don't allow braking to lower us below max speed if we started above it.
			if (bExceedingMaxSpeed && Velocity.SizeSquared() < FMath::Square(MaxPawnSpeed))
			{
				Velocity = OldVelocity.GetSafeNormal() * MaxPawnSpeed;
			}
		}
	}

	// Apply acceleration and clamp velocity magnitude.
	const float NewMaxSpeed = (IsExceedingMaxSpeed(MaxPawnSpeed)) ? Velocity.Size() : MaxPawnSpeed;
	Velocity += ControlAcceleration * FMath::Abs(Acceleration) * DeltaTime;
	Velocity = Velocity.GetClampedToMaxSize(NewMaxSpeed);

	ConsumeInputVector();
}

bool UFloatingPawnMovement::ResolvePenetrationImpl(const FVector& Adjustment, const FHitResult& Hit, const FQuat& NewRotationQuat)
{
	bPositionCorrected |= Super::ResolvePenetrationImpl(Adjustment, Hit, NewRotationQuat);
	return bPositionCorrected;
}

