// Copyright Epic Games, Inc. All Rights Reserved.

#include "Folder.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "ActorFolder.h"
#include "Misc/Paths.h"

#if WITH_EDITOR

/*
 * FFolder Helpers methods
 */

const FFolder& FFolder::GetInvalidFolder()
{
	static FFolder InvalidFolder;
	return InvalidFolder;
}

const FFolder::FRootObject& FFolder::GetInvalidRootObject()
{
	static FRootObject InvalidRootObject;
	return InvalidRootObject;
}

bool FFolder::IsRootObjectPersistentLevel(const FRootObject& Key)
{
	ULevel* Level = GetRootObjectAssociatedLevel(Key);
	return Level && Level->IsPersistentLevel();
}

TOptional<FFolder::FRootObject> FFolder::GetOptionalFolderRootObject(const ULevel* InLevel)
{
	TOptional<FFolder::FRootObject> Result;
	if (ensure(InLevel))
	{
		if (InLevel->IsPersistentLevel())
		{
			Result = FFolder::FRootObject(InLevel);
		}
		else if (ULevelStreaming* LevelStreaming = ULevelStreaming::FindStreamingLevel(InLevel))
		{
			Result = LevelStreaming->GetFolderRootObject();
		}
	}
	return Result;
}

ULevel* FFolder::GetRootObjectAssociatedLevel(const FRootObject& Key)
{
	if (const UObject* RootObjectPtr = FFolder::GetRootObjectPtr(Key))
	{
		if (const UWorld* WorldPtr = Cast<UWorld>(RootObjectPtr))
		{
			return WorldPtr->PersistentLevel;
		}
		else if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(RootObjectPtr))
		{
			return LevelInstance->GetLoadedLevel();
		}
		else if (const ULevel* Level = Cast<ULevel>(RootObjectPtr))
		{
			if (UWorld* OuterWorld = Level->GetTypedOuter<UWorld>())
			{
				return OuterWorld->PersistentLevel;
			}
		}
	}
	return nullptr;
}

FFolder FFolder::GetWorldRootFolder(UWorld* InWorld)
{
	if (InWorld)
	{
		return FFolder::GetOptionalFolderRootObject(InWorld->PersistentLevel).Get(FFolder::GetInvalidRootObject());
	}
	return FFolder::GetInvalidFolder();
}

bool FFolder::GetFolderPathsAndCommonRootObject(const TArray<FFolder>& InFolders, TArray<FName>& OutFolders, FFolder::FRootObject& OutCommonRootObject)
{
	OutFolders.Reset();

	TOptional<FFolder::FRootObject> CommonRootObject;
	for (const FFolder& Folder : InFolders)
	{
		OutFolders.Add(Folder.GetPath());
		if (!CommonRootObject.IsSet())
		{
			CommonRootObject = Folder.GetRootObject();
		}
		else if (CommonRootObject.GetValue() != Folder.GetRootObject())
		{
			OutFolders.Reset();
			CommonRootObject.Reset();
			break;
		}
	};

	OutCommonRootObject = CommonRootObject.Get(FFolder::GetInvalidRootObject());
	return CommonRootObject.IsSet();
}

/*
 * FFolder implementation
 */

const FName FFolder::GetPath() const
{
	UActorFolder* ActorFolder = nullptr;
	if (!bPathInitialized)
	{
		check(Path.IsNone());
		check(ActorFolderGuid.IsValid());
		ActorFolder = GetActorFolder();
		if (ActorFolder)
		{
			// Cache Path
			Path = ActorFolder->GetPath();
		}
		bPathInitialized = true;
	}
	return Path;
}

FFolder FFolder::GetParent() const
{
	if (UActorFolder* ActorFolder = GetActorFolder())
	{
		UActorFolder* ParentActorFolder = ActorFolder->GetParent();
		return ParentActorFolder ? ParentActorFolder->GetFolder() : FFolder(GetRootObject());
	}

	const FName ParentPath(*FPaths::GetPath(Path.ToString()));
	return FFolder(RootObject, ParentPath);
}

UActorFolder* FFolder::GetActorFolder() const
{
	ULevel* Level = GetRootObjectAssociatedLevel();
	if (Level && Level->IsUsingActorFolders())
	{
		UActorFolder* ActorFolder = Level->GetActorFolder(ActorFolderGuid, /*bSkipDeleted*/ false);
		if (ActorFolder)
		{
			return ActorFolder;
		}
		ActorFolder = Level->GetActorFolder(Path);
		if (ActorFolder)
		{
			// Cache ActorFolderGuid (GetRootObjectAssociatedLevel() can change between calls so avoid checking that it has already been set)
 			ActorFolderGuid = ActorFolder->GetGuid();
			return ActorFolder;
		}
	}
	return nullptr;
}

#endif
