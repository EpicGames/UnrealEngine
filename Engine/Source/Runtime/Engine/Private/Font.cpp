// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Font.cpp: Unreal font code.
=============================================================================*/

#include "Engine/Font.h"
#include "Engine/Texture2D.h"
#include "Fonts/FontCache.h"
#include "Framework/Application/SlateApplication.h"
#include "EngineFontServices.h"
#include "Engine/FontFace.h"
#include "HAL/FileManager.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Font)

UFontImportOptions::UFontImportOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UFont::UFont(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ScalingFactor = 1.0f;
	LegacyFontSize = 9;
}

void UFont::BeginDestroy()
{
	if (FontCacheType == EFontCacheType::Runtime && FSlateApplication::IsInitialized())
	{
		FSlateRenderer* SlateRenderer = FSlateApplication::Get().GetRenderer();
		if (SlateRenderer)
		{
			TSharedRef<FSlateFontCache> FontCache = SlateRenderer->GetFontCache();
			FontCache->FlushCompositeFont(GetInternalCompositeFont());
			FontCache->FlushObject(this);
		}
	}

	Super::BeginDestroy();
}

void UFont::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar << CharRemap;
}

void UFont::PostLoad()
{
	Super::PostLoad();

	// Cache the character count and the maximum character height for this font
	CacheCharacterCountAndMaxCharHeight();
	
#if WITH_EDITOR
	for( int32 TextureIndex = 0 ; TextureIndex < Textures.Num() ; ++TextureIndex )
	{
		UTexture2D* Texture = Textures[TextureIndex];
		if( Texture )
		{	
			Texture->SetFlags(RF_Public);
			
			// Texture changes like this should NOT be done in PostLoad

			bool bTextureChanged = false;
			if ( Texture->LODGroup != TEXTUREGROUP_UI )
			{
				bTextureChanged = true;
				Texture->PreEditChange(nullptr);

				Texture->LODGroup = TEXTUREGROUP_UI;
			}

			// Fix up compression type for distance field fonts.
			if (Texture->CompressionSettings == TC_Displacementmap && !Texture->SRGB)
			{
				if ( ! bTextureChanged )
				{
					bTextureChanged = true;
					Texture->PreEditChange(nullptr);
				}

				Texture->CompressionSettings = TC_DistanceFieldFont;
			}

			if ( bTextureChanged )
			{
				Texture->PostEditChange();
			}
		}
	}
#endif

#if WITH_EDITORONLY_DATA
	if (FontCacheType == EFontCacheType::Runtime)
	{
		auto UpgradeLegacyData = [this](const int32 InTypefaceIndex, const FName InTypefaceName, FFontData& InFontData)
		{
			if (!InFontData.HasLegacyData())
			{
				return;
			}

			// Need to give these things useful and unique names, as they will be used as the .ufont name when cooking
			const FString FontFaceObjectDisplayName = FString::Printf(TEXT("%s_%d_%s"), *GetName(), InTypefaceIndex, *InTypefaceName.ToString());
			FName FontFaceObjectName = MakeObjectNameFromDisplayLabel(FontFaceObjectDisplayName, NAME_None);
			if (FindObject<UFontFace>(this, *FontFaceObjectName.ToString()))
			{
				FontFaceObjectName = MakeUniqueObjectName(this, UFontFace::StaticClass(), FontFaceObjectName);
			}

			InFontData.ConditionalUpgradeFontDataToBulkData(this);
			InFontData.ConditionalUpgradeBulkDataToFontFace(this, UFontFace::StaticClass(), FontFaceObjectName);
		};

		int32 TypefaceIndex = 0;
		FCompositeFont& MutableInternalCompositeFont = GetMutableInternalCompositeFont();
		for (FTypefaceEntry& TypefaceEntry : MutableInternalCompositeFont.DefaultTypeface.Fonts)
		{
			UpgradeLegacyData(TypefaceIndex, TypefaceEntry.Name, TypefaceEntry.Font);
		}

		for (FCompositeSubFont& SubFont : MutableInternalCompositeFont.SubTypefaces)
		{
			++TypefaceIndex;
			for (FTypefaceEntry& TypefaceEntry : SubFont.Typeface.Fonts)
			{
				UpgradeLegacyData(TypefaceIndex, TypefaceEntry.Name, TypefaceEntry.Font);
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UFont::CacheCharacterCountAndMaxCharHeight()
{
	// Cache the number of characters in the font.  Obviously this is pretty simple, but note that it will be
	// computed differently for MultiFonts.  We need to cache it so that we have it available in inline functions
	NumCharacters = Characters.Num();

	// Cache maximum character height
	MaxCharHeight.Reset();
	int32 MaxCharHeightForThisFont = 1;
	for( int32 CurCharNum = 0; CurCharNum < NumCharacters; ++CurCharNum )
	{
		MaxCharHeightForThisFont = FMath::Max( MaxCharHeightForThisFont, Characters[ CurCharNum ].VSize );
	}

	// Add to the array
	MaxCharHeight.Add( MaxCharHeightForThisFont );
}

TCHAR UFont::RemapChar(TCHAR CharCode) const
{
	const uint16 UCode = CharCast<UCS2CHAR>(CharCode);
	if ( IsRemapped )
	{
		// currently, fonts are only remapped if they contain Unicode characters.
		// For remapped fonts, all characters in the CharRemap map are valid, so
		// if the characters exists in the map, it's safe to use - otherwise, return
		// the null character (an empty square on windows)
		const uint16* FontChar = CharRemap.Find(UCode);
		if ( FontChar == NULL )
			return NULLCHARACTER;

		return (TCHAR)*FontChar;
	}

	// Otherwise, our Characters array will contains 256 members, and is
	// a one-to-one mapping of character codes to array indexes, though
	// not every character is a valid character.
	if ( UCode >= NumCharacters )
	{
		return NULLCHARACTER;
	}

	// If the character's size is 0, it's non-printable or otherwise unsupported by
	// the font.  Return the default null character (an empty square on windows).
	if ( !Characters.IsValidIndex(UCode) || (Characters[UCode].VSize == 0 && UCode >= TEXT(' ')) )
	{
		return NULLCHARACTER;
	}

	return CharCode;
}

void UFont::GetCharSize(TCHAR InCh, float& Width, float& Height) const
{
	Width = Height = 0.f;

	switch(FontCacheType)
	{
	case EFontCacheType::Offline:
		{
			const int32 Ch = (int32)RemapChar(InCh);
			if( Ch < Characters.Num() )
			{
				const FFontCharacter& Char = Characters[Ch];
				if( Char.TextureIndex < Textures.Num() && Textures[Char.TextureIndex] != NULL )
				{
					Width = Char.USize;

					// The height of the character will always be the maximum height of any character in this
					// font.  This ensures consistent vertical alignment of text.  For example, we don't want
					// vertically centered text to visually shift up and down as characters are added to a string.
					// NOTE: This also gives us consistent alignment with fonts generated by the legacy importer.
					const int32 MultiFontIndex = Ch / NumCharacters;
					Height = MaxCharHeight[ MultiFontIndex ];
				}
			}
		}
		break;

	case EFontCacheType::Runtime:
		{
			TSharedPtr<FSlateFontCache> FontCache = FEngineFontServices::Get().GetFontCache();

			if (FontCache.IsValid())
			{
				const float FontScale = 1.0f;
				const FSlateFontInfo LegacyFontInfo = GetLegacySlateFontInfo();
				FCharacterList& CharacterList = FontCache->GetCharacterList(LegacyFontInfo, FontScale);
				const FCharacterEntry& Entry = CharacterList.GetCharacter(InCh, LegacyFontInfo.FontFallback);

				Width = Entry.XAdvance;

				// The height of the character will always be the maximum height of any character in this font
				Height = CharacterList.GetMaxHeight();
			}
		}
		break;

	default:
		break;
	}
}

int8 UFont::GetCharKerning(TCHAR First, TCHAR Second) const
{
	switch(FontCacheType)
	{
	case EFontCacheType::Offline:
		{
			return static_cast<int8>(Kerning);
		}

	case EFontCacheType::Runtime:
		{
			TSharedPtr<FSlateFontCache> FontCache = FEngineFontServices::Get().GetFontCache();

			if (FontCache.IsValid())
			{
				const float FontScale = 1.0f;
				const FSlateFontInfo LegacyFontInfo = GetLegacySlateFontInfo();
				FCharacterList& CharacterList = FontCache->GetCharacterList(LegacyFontInfo, FontScale);

				return CharacterList.GetKerning(First, Second, LegacyFontInfo.FontFallback);
			}
		}
		break;

	default:
		break;
	}

	return 0;
}

int16 UFont::GetCharHorizontalOffset(TCHAR InCh) const
{
	if(FontCacheType == EFontCacheType::Runtime)
	{
		TSharedPtr<FSlateFontCache> FontCache = FEngineFontServices::Get().GetFontCache();

		if (FontCache.IsValid())
		{
			const float FontScale = 1.0f;
			const FSlateFontInfo LegacyFontInfo = GetLegacySlateFontInfo();
			FCharacterList& CharacterList = FontCache->GetCharacterList(LegacyFontInfo, FontScale);
			const FCharacterEntry& Entry = CharacterList.GetCharacter(InCh, LegacyFontInfo.FontFallback);

			return Entry.HorizontalOffset;
		}
	}

	return 0;
}

int32 UFont::GetStringSize( const TCHAR *Text ) const
{
	int32 Width, Height;
	GetStringHeightAndWidth( Text, Height, Width );
	return Width;
}

int32 UFont::GetStringHeightSize( const TCHAR *Text ) const
{
	int32 Width, Height;
	GetStringHeightAndWidth( Text, Height, Width );
	return Height;
}

float UFont::GetMaxCharHeight() const
{
	float MaxHeight = 0.0f;

	switch(FontCacheType)
	{
	case EFontCacheType::Offline:
		{
			// @todo: Provide a version of this function that supports multi-fonts properly.  It should take a
			//    HeightTest parameter and report the appropriate multi-font MaxCharHeight value back.
			int32 MaxCharHeightForAllMultiFonts = 1;
			for( int32 CurMultiFontIndex = 0; CurMultiFontIndex < MaxCharHeight.Num(); ++CurMultiFontIndex )
			{
				MaxCharHeightForAllMultiFonts = FMath::Max( MaxCharHeightForAllMultiFonts, MaxCharHeight[ CurMultiFontIndex ] );
			}
			MaxHeight = MaxCharHeightForAllMultiFonts;
		}
		break;

	case EFontCacheType::Runtime:
		{
			TSharedPtr<FSlateFontCache> FontCache = FEngineFontServices::Get().GetFontCache();
			
			if (FontCache.IsValid())
			{
				const float FontScale = 1.0f;
				const FSlateFontInfo LegacyFontInfo(this, LegacyFontSize);
				FCharacterList& CharacterList = FontCache->GetCharacterList(LegacyFontInfo, FontScale);

				MaxHeight = CharacterList.GetMaxHeight();
			}
		}
		break;

	default:
		break;
	}

	return MaxHeight;
}

void UFont::GetStringHeightAndWidth( const FString& InString, int32& Height, int32& Width ) const
{
	GetStringHeightAndWidth( *InString, Height, Width );
}

void UFont::GetStringHeightAndWidth( const TCHAR *Text, int32& Height, int32& Width ) const
{
	float TotalWidth = 0.0f;
	float TotalHeight = 0.0f;
	const TCHAR* PrevChar = nullptr;
	while( *Text )
	{
		float TmpWidth, TmpHeight;
		GetCharSize( *Text, TmpWidth, TmpHeight );

		int8 CharKerning = 0;
		if (PrevChar)
		{
			CharKerning = GetCharKerning( *PrevChar, *Text );
		}

		TotalWidth += TmpWidth + CharKerning;
		TotalHeight = FMath::Max( TotalHeight, TmpHeight );

		PrevChar = Text++;
	}

	Width = FMath::CeilToInt( TotalWidth );
	Height = FMath::CeilToInt( TotalHeight );
}

FSlateFontInfo UFont::GetLegacySlateFontInfo() const
{
	return FSlateFontInfo(this, static_cast<float>(LegacyFontSize), LegacyFontName);
}

void UFont::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Nested Textures/FontFaces get included with the general subobject inclusion for EstimatedTotal
	// We do need to handle nested lazy load data

	if (FontCacheType == EFontCacheType::Runtime && CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::EstimatedTotal)
	{
		auto GetTypefaceResourceSize = [&CumulativeResourceSize](const FTypeface& Typeface)
		{
			for (const FTypefaceEntry& TypefaceEntry : Typeface.Fonts)
			{
				const UFontFace* FontFace = Cast<const UFontFace>(TypefaceEntry.Font.GetFontFaceAsset());
				if (!FontFace && TypefaceEntry.Font.GetLoadingPolicy() == EFontLoadingPolicy::LazyLoad)
				{
					const int64 FileSize = IFileManager::Get().FileSize(*TypefaceEntry.Font.GetFontFilename());
					if (FileSize > 0)
					{
						CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FileSize);
					}
				}
			}
		};

		// Sum the contained font data sizes
		const FCompositeFont& InternalCompositeFont = GetInternalCompositeFont();
		GetTypefaceResourceSize(InternalCompositeFont.DefaultTypeface);
		for (const FCompositeSubFont& SubTypeface : InternalCompositeFont.SubTypefaces)
		{
			GetTypefaceResourceSize(SubTypeface.Typeface);
		}
	}
}

const FCompositeFont* UFont::GetCompositeFont() const
{
	if (FontCacheType == EFontCacheType::Runtime)
	{
		switch (RuntimeFontSource)
		{
			case ERuntimeFontSource::Asset:
				return &GetInternalCompositeFont();

			case ERuntimeFontSource::CoreStyleDefault:
				// Note: Returning this as a raw pointer is safe (for use on the stack), as FLegacySlateFontInfoCache keeps the FCompositeFont alive as long as Slate is initialized
				return &FCoreStyle::GetDefaultFont().Get();

			default:
				break;
		}
	}

	return nullptr;
}
