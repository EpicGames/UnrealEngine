// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/FontFace.h"
#include "Engine/Font.h"
#include "Engine/UserInterfaceSettings.h"
#include "EditorFramework/AssetImportData.h"
#include "Misc/Paths.h"
#include "Framework/Application/SlateApplication.h"
#include "Fonts/FontCache.h"
#include "Rendering/SlateRenderer.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Misc/FileHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FontFace)

DEFINE_LOG_CATEGORY_STATIC(LogFontFace, Log, All);

namespace UE::Private::FontFace
{
	const FString& GetStandaloneFontFaceDirectory()
	{
		static const FString FontFaceDirectory = FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("FontFaces"));
		return FontFaceDirectory;
	}
}

static bool GSimulateCookedFontFilesInStandalone = false;
FAutoConsoleVariableRef CVarSimulateCookedFontFilesInStandalone(
	TEXT("UI.Slate.Font.SimulateCookedFontFilesInStandalone"),
	GSimulateCookedFontFilesInStandalone,
	TEXT("If true when running in standalone, the embedded font face data will be written to the Intermediate project folder and loaded on demand. This simulates the loading of .ufont files in a cooked build."));

TAutoConsoleVariable<FString> CVarFontFaceDistanceFieldRasterizationMode(
	TEXT("UI.SlateSDFText.RasterizationMode"),
	"Bitmap",
	TEXT("Sets the rasterization mode of font faces with distance field rasterization enabled. Possible values are: Bitmap, Msdf, Sdf, SdfApproximation."),
	ECVF_Preview);

TAutoConsoleVariable<int32> CVarFontFaceDistanceFieldResolutionLevel(
	TEXT("UI.SlateSDFText.ResolutionLevel"),
	2,
	TEXT("Sets the resolution level (1 = low, 2 = medium, 3 = high) of font faces with distance field rasterization enabled."),
	ECVF_Preview);

EFontRasterizationMode GetDeviceFontFaceDistanceFieldRasterizationMode()
{
	const int64 EnumVal = StaticEnum<EFontRasterizationMode>()->GetValueByNameString(CVarFontFaceDistanceFieldRasterizationMode.GetValueOnAnyThread());
	if (EnumVal != INDEX_NONE)
	{
		return (EFontRasterizationMode) EnumVal;
	}
	UE_LOGF(LogFontFace, Warning, "Unexpected value of CVar UI.SlateSDFText.RasterizationMode - falling back to Bitmap.");
	return EFontRasterizationMode::Bitmap;
}

UFontFace::UFontFace()
	: AscendOverriddenValue(0)
	, bIsAscendOverridden(false)
	, DescendOverriddenValue(0)
	, bIsDescendOverridden(false)
	, StrikeBrushHeightPercentage(60)
	, FontFaceData(FFontFaceData::MakeFontFaceData())

{
}

void UFontFace::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYNAME(TEXT("FontFaceData"));
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Super::Serialize(Ar);

	bool bCooked = Ar.IsCooking();
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::AddedCookedBoolFontFaceAssets)
	{
		Ar << bCooked;
	}

	if (Ar.IsLoading())
	{
		if (FPlatformProperties::RequiresCookedData() || bCooked)
		{
			SourceFilename = GetCookedFilename();
		}

		if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::AddedInlineFontFaceAssets)
		{
#if WITH_EDITORONLY_DATA
			// Port the old property data into the shared instance
			FontFaceData->SetData(MoveTemp(FontFaceData_DEPRECATED));
#endif // WITH_EDITORONLY_DATA
		}
		else
		{
			bool bLoadInlineData = false;
			Ar << bLoadInlineData;

			if (bLoadInlineData)
			{
				if (FontFaceData->HasData())
				{
					// If we already have data, make a new instance in-case the existing one is being referenced by the font cache
					FontFaceData = FFontFaceData::MakeFontFaceData();
				}
				FontFaceData->Serialize(Ar);
			}
		}

#if WITH_EDITORONLY_DATA
		// This will be executed in Postload instead to avoid thread-safety issue
		// when the serialization comes from the loader, which could run on any thread.
		if (!HasAnyFlags(RF_NeedPostLoad))
		{
			CacheSubFaces();
		}
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
		// Only save the inline data in a cooked build if we're using the inline loading policy
		bool bSaveInlineData = LoadingPolicy == EFontLoadingPolicy::Inline || !Ar.IsCooking();
		Ar << bSaveInlineData;

		if (bSaveInlineData)
		{
			FontFaceData->Serialize(Ar);
		}
	}
}

void UFontFace::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	const bool bCountInlineData = WITH_EDITORONLY_DATA || LoadingPolicy == EFontLoadingPolicy::Inline;
	// Only count the memory size for fonts that will be loaded
	if (bCountInlineData)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FontFaceData->GetData().GetAllocatedSize());
	}
	// only get size if lazy loading.  Resident font memory wont exist for EFontLoadingPolicy::Stream
	else if(LoadingPolicy == EFontLoadingPolicy::LazyLoad && FSlateApplication::IsInitialized())
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FSlateApplication::Get().GetRenderer()->GetFontCache()->GetFontDataAssetResidentMemory(this));
	}

}

void UFontFace::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	CacheSubFaces();
#endif // WITH_EDITORONLY_DATA

	UpdateDeviceRasterizationSettings();

#if WITH_EDITOR
	// For the regular Editor, all font faces need to be loaded for editing and the font data is embedded.
	// In standalone, we write out the font data for lazily loaded fonts to the Intermediate project folder and replace the filename of the font face.
	// This causes lazily loaded fonts to actually be loaded from disk to simulate the behavior in a cooked build.
	if (!GIsEditor && !GIsServer && GSimulateCookedFontFilesInStandalone && LoadingPolicy == EFontLoadingPolicy::LazyLoad)
	{
		if (FontFaceData->HasData())
		{
			FString Filename = GetStandaloneFilename();
			FString DirectoryPath = FPaths::GetPath(Filename);
			if (!IPlatformFile::GetPlatformPhysical().DirectoryExists(*DirectoryPath))
			{
				UE_LOGF(LogFontFace, Display, "Standalone font face directory '%ls' does not exist. Creating directory.", *DirectoryPath);
				if (!IPlatformFile::GetPlatformPhysical().CreateDirectoryTree(*DirectoryPath))
				{
					UE_LOGF(LogFontFace, Warning, "Unable to create standalone font face directory '%ls'. Font face '%ls' will not be written and async font loading of font face will be skipped.", *DirectoryPath, *Filename);
					return;
				}
			}

			if (FFileHelper::SaveArrayToFile(FontFaceData->GetData(), *Filename))
			{
				UE_LOGF(LogFontFace, Display, "Written font face file '%ls' to simulate loading of cooked font files in standalone mode.", *Filename);
				SourceFilename = MoveTemp(Filename);
				// We clear the data in the FFontFaceData so we force a synchronous load of the data 
				TArray<uint8> EmptyData;
				FontFaceData->SetData(MoveTemp(EmptyData));
			}
			else
			{
				UE_LOGF(LogFontFace, Warning, "Failed to write font face file '%ls' to simulate loading of cooked font files in standalone mode.", *Filename);
			}
		}
	}
#endif
}

#if WITH_EDITOR
FString UFontFace::GetStandaloneFilename() const
{
	// This should ever only be called when in standalone 
	ensure(!GIsEditor);
	// This implementation follows UFontFace::GetCookedFilename. If that implementation changes, it is best to also adapt this to the change 
	FString PackageName = GetOutermost()->GetName();
	PackageName = FPackageName::GetLocalizedPackagePath(PackageName);

	return FPaths::Combine(UE::Private::FontFace::GetStandaloneFontFaceDirectory(), PackageName) / GetName() + TEXT(".ufont");
}

void UFontFace::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateDeviceRasterizationSettings();

#if WITH_EDITORONLY_DATA
	CacheSubFaces();
#endif // WITH_EDITORONLY_DATA

	FSlateApplication::Get().GetRenderer()->FlushFontCache(TEXT("UFontFace::PostEditChangeProperty"));
}

void UFontFace::PostEditUndo()
{
	Super::PostEditUndo();

#if WITH_EDITORONLY_DATA
	CacheSubFaces();
#endif // WITH_EDITORONLY_DATA

	FSlateApplication::Get().GetRenderer()->FlushFontCache(TEXT("UFontFace::PostEditUndo"));
}

void UFontFace::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UFontFace::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	FAssetImportInfo ImportInfo;
	ImportInfo.Insert(FAssetImportInfo::FSourceFile(SourceFilename));
	Context.AddTag(FAssetRegistryTag(SourceFileTagName(), ImportInfo.ToJson(), FAssetRegistryTag::TT_Hidden));
}

void UFontFace::CookAdditionalFilesOverride(const TCHAR* PackageFilename, const ITargetPlatform* TargetPlatform,
	TFunctionRef<void(const TCHAR* Filename, void* Data, int64 Size)> WriteAdditionalFile)
{
	if (LoadingPolicy != EFontLoadingPolicy::Inline)
	{
		// Iterative COTF can't handle the .ufont files generated when this UFontFace is within a UFont asset (rather than its own asset)
		if (UFont* OuterFont = GetTypedOuter<UFont>())
		{
			UE_LOGF(LogFontFace, Warning, 
				"The font asset '%ls' contains nested font faces which can cause issues for iterative cook-on-the-fly."
					"Please edit the font asset and split the font faces into their own assets.",
				*OuterFont->GetPathName());
		}

		// We replace the package name with the cooked font face name
		// Note: This must match the replacement logic in UFontFace::GetCookedFilename
		const FString CookedFontFilename = FPaths::GetPath(PackageFilename) / GetName() + TEXT(".ufont");
		
		TArray<uint8> Data;
		FMemoryWriter Ar(Data, true);
		FontFaceData->Serialize(Ar);
		
		const int32 NumBytes = Data.Num() * Data.GetTypeSize();
		WriteAdditionalFile(*CookedFontFilename, (void*)Data.GetData(), NumBytes);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UFontFace::CacheSubFaces()
{
	if (FSlateApplication::IsInitialized())
	{
		SubFaces = FSlateApplication::Get().GetRenderer()->GetFontCache()->GetAvailableFontSubFaces(FontFaceData);
#if WITH_EDITOR && WITH_FREETYPE
		if (bEnableDistanceFieldRendering && GetDefault<UUserInterfaceSettings>()->bEnableDistanceFieldFontRasterization && IsSlateSdfTextFeatureEnabled())
		{
			FSlateApplication::Get().GetRenderer()->GetFontCache()->EnsurePreprocessedFontGeometry(FontFaceData);
		}
#endif
	}
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
void UFontFace::InitializeFromBulkData(const FString& InFilename, const EFontHinting InHinting, const void* InBulkDataPtr, const int32 InBulkDataSizeBytes)
{
	check(InBulkDataPtr && InBulkDataSizeBytes > 0 && !FontFaceData->HasData());

	SourceFilename = InFilename;
	Hinting = InHinting;
	LoadingPolicy = EFontLoadingPolicy::LazyLoad;
	
	TArray<uint8> FontData;
	FontData.Append(static_cast<const uint8*>(InBulkDataPtr), InBulkDataSizeBytes);
	FontFaceData->SetData(MoveTemp(FontData));

	CacheSubFaces();
}
#endif // WITH_EDITORONLY_DATA

const FString& UFontFace::GetFontFilename() const
{
	return SourceFilename;
}

EFontHinting UFontFace::GetHinting() const
{
	return Hinting;
}

EFontLoadingPolicy UFontFace::GetLoadingPolicy() const
{
	return LoadingPolicy;
}

EFontLayoutMethod UFontFace::GetLayoutMethod() const
{
	return LayoutMethod;
}

bool UFontFace::IsAscendOverridden() const
{
	return bIsAscendOverridden;
}

int32 UFontFace::GetAscendOverriddenValue() const
{
	return AscendOverriddenValue;
}

bool UFontFace::IsDescendOverridden() const
{
	return bIsDescendOverridden;
}

int32 UFontFace::GetDescendOverriddenValue() const
{
	return DescendOverriddenValue;
}

int32 UFontFace::GetStrikeBrushHeightPercentage() const
{
	return StrikeBrushHeightPercentage;
}

FFontFaceDataConstRef UFontFace::GetFontFaceData() const
{
	return FontFaceData;
}

FFontRasterizationSettings UFontFace::GetRasterizationSettings() const
{
	return DeviceRasterizationSettings;
}

FString UFontFace::GetCookedFilename() const
{
	// UFontFace assets themselves can't be localized, however that doesn't mean the package they're in isn't localized (ie, when they're upgraded into a UFont asset)
	FString PackageName = GetOutermost()->GetName();
	if (!GIsEditor)
	{
		PackageName = FPackageName::GetLocalizedPackagePath(PackageName);
	}
	
	// Note: This must match the replacement logic in UFontFace::CookAdditionalFiles
	const FString PackageFilename = FPackageName::LongPackageNameToFilename(PackageName, TEXT(".uasset"));
	return FPaths::GetPath(PackageFilename) / GetName() + TEXT(".ufont");
}

void UFontFace::UpdateDeviceRasterizationSettings()
{
	DeviceRasterizationSettings = FFontRasterizationSettings();
	if (bEnableDistanceFieldRendering && GetDefault<UUserInterfaceSettings>()->bEnableDistanceFieldFontRasterization)
	{
		DeviceRasterizationSettings.Mode = GetDeviceFontFaceDistanceFieldRasterizationMode();
		if (PlatformRasterizationModeOverrides.IsSet())
		{
			switch (DeviceRasterizationSettings.Mode)
			{
				default:
					checkNoEntry();
					break;
				case EFontRasterizationMode::Bitmap:
					break;
				case EFontRasterizationMode::Msdf:
					DeviceRasterizationSettings.Mode = PlatformRasterizationModeOverrides->MsdfOverride;
					break;
				case EFontRasterizationMode::Sdf:
					DeviceRasterizationSettings.Mode = PlatformRasterizationModeOverrides->SdfOverride;
					break;
				case EFontRasterizationMode::SdfApproximation:
					DeviceRasterizationSettings.Mode = PlatformRasterizationModeOverrides->SdfApproximationOverride;
					break;
			}
		}

		const bool bMultiChannel = DeviceRasterizationSettings.Mode == EFontRasterizationMode::Msdf;
		const int32 DeviceResolutionLevel = CVarFontFaceDistanceFieldResolutionLevel.GetValueOnAnyThread();
		if (DeviceResolutionLevel <= 1) // Low
		{
			DeviceRasterizationSettings.DistanceFieldPpem = bMultiChannel ? MinMultiDistanceFieldPpem : MinDistanceFieldPpem;
		}
		else if (DeviceResolutionLevel >= 3) // High
		{
			DeviceRasterizationSettings.DistanceFieldPpem = bMultiChannel ? MaxMultiDistanceFieldPpem : MaxDistanceFieldPpem;
		}
		else // 2 = Medium
		{
			DeviceRasterizationSettings.DistanceFieldPpem = bMultiChannel ? MidMultiDistanceFieldPpem : MidDistanceFieldPpem;
		}
	}
}
