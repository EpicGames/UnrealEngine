// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUBitonicSort.cpp: Implementation of bitonic sort on the GPU.
=============================================================================*/

#include "GPUBitonicSort.h"
#include "GPUSort.h"
#include "GPUSortManager.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIBreadcrumbs.h"
#include "RenderingThread.h"
#include "RHIContext.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterUtils.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

/*------------------------------------------------------------------------------
	Compile-time sort parameters. Must match the shader defines.
------------------------------------------------------------------------------*/

// Tuned for a 32 KB shared-memory budget.
#define BITONIC_THREAD_COUNT 512
#define BITONIC_ELEMENTS_PER_THREAD 2
#define BITONIC_BLOCK_SIZE (BITONIC_THREAD_COUNT * BITONIC_ELEMENTS_PER_THREAD)

// Max group count and coalesced access.
#define BITONIC_GLOBAL_MERGE_EPT 1

// Padded footprint at BLOCK_SIZE=1024: (1024+32) * 4 * 2  ~ 8.25 KB.
static_assert((BITONIC_BLOCK_SIZE + (BITONIC_BLOCK_SIZE >> 5)) * sizeof(uint32) * 2 <= 32768, "Bitonic sort shared memory with padding exceeds 32 KB limit");

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FBitonicSortParameters, )
	SHADER_PARAMETER(uint32, BitonicSort_Count)
	SHADER_PARAMETER(uint32, BitonicSort_PaddedCount)
	SHADER_PARAMETER(uint32, BitonicSort_StageBlockSize)
	SHADER_PARAMETER(uint32, BitonicSort_StepBlockSize)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FBitonicSortParameters, "BitonicSortUB");

using FBitonicSortUniformBufferRef = TUniformBufferRef<FBitonicSortParameters>;

class FBitonicUseWaveOps : SHADER_PERMUTATION_BOOL("BITONIC_USE_WAVE_OPS");

static bool UseWaveOps(EShaderPlatform ShaderPlatform)
{
	if ((GRHIGlobals.MinimumWaveSize > 32) || (GRHIGlobals.MaximumWaveSize < 32))
	{
		return false;
	}

	const ERHIFeatureSupport WaveOpsSupport = FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(ShaderPlatform);
	return (GRHISupportsWaveOperations && WaveOpsSupport == ERHIFeatureSupport::RuntimeDependent)
		|| (WaveOpsSupport == ERHIFeatureSupport::RuntimeGuaranteed);
}

static void SetBitonicSortShaderCompilerEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("BITONIC_THREAD_COUNT"), BITONIC_THREAD_COUNT);
	OutEnvironment.SetDefine(TEXT("BITONIC_ELEMENTS_PER_THREAD"), BITONIC_ELEMENTS_PER_THREAD);
	OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);

	// Activates the wave-shuffle tier in BitonicSubSteps (zero-barrier medium strides).
	using FPermDomain = TShaderPermutationDomain<FBitonicUseWaveOps>;
	const bool bUseWaveOps = FPermDomain(Parameters.PermutationId).Get<FBitonicUseWaveOps>();
	if (bUseWaveOps)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WaveOperations);
	}
}

class FBitonicSortLocalSortCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FBitonicSortLocalSortCS, Global);

public:

	using FPermutationDomain = TShaderPermutationDomain<FBitonicUseWaveOps>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const ERHIFeatureSupport WaveOpsSupport = FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(Parameters.Platform);
		const bool bWaveOpsPermutation = FPermutationDomain(Parameters.PermutationId).Get<FBitonicUseWaveOps>();
		if (bWaveOpsPermutation)
		{
			if (WaveOpsSupport == ERHIFeatureSupport::Unsupported)
			{
				return false;
			}
		}
		else if ((WaveOpsSupport == ERHIFeatureSupport::RuntimeGuaranteed) &&
			(FDataDrivenShaderPlatformInfo::GetMinimumWaveSize(Parameters.Platform) == 32) &&
			(FDataDrivenShaderPlatformInfo::GetMaximumWaveSize(Parameters.Platform) == 32))
		{
			return false;
		}

		return true;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FBitonicUseWaveOps>() && !UseWaveOps(GMaxRHIShaderPlatform))
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		return EShaderPermutationPrecacheRequest::Precached;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("BITONIC_LOCAL_SORT"), 1);
		SetBitonicSortShaderCompilerEnvironment(Parameters, OutEnvironment);
	}

	FBitonicSortLocalSortCS() = default;

	explicit FBitonicSortLocalSortCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		InKeys.Bind(Initializer.ParameterMap, TEXT("InKeys"));
		InValues.Bind(Initializer.ParameterMap, TEXT("InValues"));
		OutKeys.Bind(Initializer.ParameterMap, TEXT("OutKeys"));
		OutValues.Bind(Initializer.ParameterMap, TEXT("OutValues"));
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FBitonicSortUniformBufferRef& UniformBuffer,
		FRHIShaderResourceView* InKeysSRV,
		FRHIUnorderedAccessView* InValuesUAV,
		FRHIUnorderedAccessView* OutKeysUAV,
		FRHIUnorderedAccessView* OutValuesUAV)
	{
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FBitonicSortParameters>(), UniformBuffer);
		SetSRVParameter(BatchedParameters, InKeys, InKeysSRV);
		SetUAVParameter(BatchedParameters, InValues, InValuesUAV);
		SetUAVParameter(BatchedParameters, OutKeys, OutKeysUAV);
		SetUAVParameter(BatchedParameters, OutValues, OutValuesUAV);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetSRVParameter(BatchedUnbinds, InKeys);
		UnsetUAVParameter(BatchedUnbinds, InValues);
		UnsetUAVParameter(BatchedUnbinds, OutKeys);
		UnsetUAVParameter(BatchedUnbinds, OutValues);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InKeys);
	LAYOUT_FIELD(FShaderResourceParameter, InValues);
	LAYOUT_FIELD(FShaderResourceParameter, OutKeys);
	LAYOUT_FIELD(FShaderResourceParameter, OutValues);
};
IMPLEMENT_SHADER_TYPE(, FBitonicSortLocalSortCS, TEXT("/Engine/Private/BitonicSort.usf"), TEXT("BitonicSort_LocalSort"), SF_Compute);

class FBitonicSortGlobalMergeCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FBitonicSortGlobalMergeCS, Global);

public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("BITONIC_GLOBAL_MERGE"), 1);
		OutEnvironment.SetDefine(TEXT("BITONIC_GLOBAL_MERGE_EPT"), BITONIC_GLOBAL_MERGE_EPT);
		SetBitonicSortShaderCompilerEnvironment(Parameters, OutEnvironment);
	}

	FBitonicSortGlobalMergeCS() = default;

	explicit FBitonicSortGlobalMergeCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		InOutKeys.Bind(Initializer.ParameterMap, TEXT("InOutKeys"));
		InOutValues.Bind(Initializer.ParameterMap, TEXT("InOutValues"));
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FBitonicSortUniformBufferRef& UniformBuffer,
		FRHIUnorderedAccessView* KeysUAV,
		FRHIUnorderedAccessView* ValuesUAV)
	{
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FBitonicSortParameters>(), UniformBuffer);
		SetUAVParameter(BatchedParameters, InOutKeys, KeysUAV);
		SetUAVParameter(BatchedParameters, InOutValues, ValuesUAV);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetUAVParameter(BatchedUnbinds, InOutKeys);
		UnsetUAVParameter(BatchedUnbinds, InOutValues);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InOutKeys);
	LAYOUT_FIELD(FShaderResourceParameter, InOutValues);
};
IMPLEMENT_SHADER_TYPE(, FBitonicSortGlobalMergeCS, TEXT("/Engine/Private/BitonicSort.usf"), TEXT("BitonicSort_GlobalMerge"), SF_Compute);

class FBitonicSortLocalMergeCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FBitonicSortLocalMergeCS, Global);

public:

	using FPermutationDomain = TShaderPermutationDomain<FBitonicUseWaveOps>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const ERHIFeatureSupport WaveOpsSupport = FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(Parameters.Platform);
		const bool bWaveOpsPermutation = FPermutationDomain(Parameters.PermutationId).Get<FBitonicUseWaveOps>();
		if (bWaveOpsPermutation)
		{
			if (WaveOpsSupport == ERHIFeatureSupport::Unsupported)
			{
				return false;
			}
		}
		else if ((WaveOpsSupport == ERHIFeatureSupport::RuntimeGuaranteed) &&
			(FDataDrivenShaderPlatformInfo::GetMinimumWaveSize(Parameters.Platform) == 32) &&
			(FDataDrivenShaderPlatformInfo::GetMaximumWaveSize(Parameters.Platform) == 32))
		{
			return false;
		}

		return true;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FBitonicUseWaveOps>() && !UseWaveOps(GMaxRHIShaderPlatform))
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		return EShaderPermutationPrecacheRequest::Precached;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("BITONIC_LOCAL_MERGE"), 1);
		SetBitonicSortShaderCompilerEnvironment(Parameters, OutEnvironment);
	}

	FBitonicSortLocalMergeCS() = default;

	explicit FBitonicSortLocalMergeCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		InKeys.Bind(Initializer.ParameterMap, TEXT("InKeys"));
		InValues.Bind(Initializer.ParameterMap, TEXT("InValues"));
		OutKeys.Bind(Initializer.ParameterMap, TEXT("OutKeys"));
		OutValues.Bind(Initializer.ParameterMap, TEXT("OutValues"));
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FBitonicSortUniformBufferRef& UniformBuffer,
		FRHIShaderResourceView* InKeysSRV,
		FRHIUnorderedAccessView* InValuesUAV,
		FRHIUnorderedAccessView* OutKeysUAV,
		FRHIUnorderedAccessView* OutValuesUAV)
	{
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FBitonicSortParameters>(), UniformBuffer);
		SetSRVParameter(BatchedParameters, InKeys, InKeysSRV);
		SetUAVParameter(BatchedParameters, InValues, InValuesUAV);
		SetUAVParameter(BatchedParameters, OutKeys, OutKeysUAV);
		SetUAVParameter(BatchedParameters, OutValues, OutValuesUAV);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetSRVParameter(BatchedUnbinds, InKeys);
		UnsetUAVParameter(BatchedUnbinds, InValues);
		UnsetUAVParameter(BatchedUnbinds, OutKeys);
		UnsetUAVParameter(BatchedUnbinds, OutValues);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InKeys);
	LAYOUT_FIELD(FShaderResourceParameter, InValues);
	LAYOUT_FIELD(FShaderResourceParameter, OutKeys);
	LAYOUT_FIELD(FShaderResourceParameter, OutValues);
};
IMPLEMENT_SHADER_TYPE(, FBitonicSortLocalMergeCS, TEXT("/Engine/Private/BitonicSort.usf"), TEXT("BitonicSort_LocalMerge"), SF_Compute);


// Dispatches one bitonic pass: reads SortBuffers[SrcBuf] and writes in SortBuffers[DstBuf].
template<typename TShaderType>
static void DispatchBitonicKernel(
	FRHICommandList& RHICmdList,
	FComputeShaderRHIRef ComputeShader,
	TShaderMapRef<TShaderType>& ShaderRef,
	const FBitonicSortUniformBufferRef& UniformBuffer,
	FGPUSortBuffers& SortBuffers,
	int32 SrcBuf,
	int32 DstBuf,
	FRHIUnorderedAccessView* ValuesReadUAV,
	FRHIUnorderedAccessView* ValuesWriteUAV,
	int32 GroupCount)
{
	checkf(ValuesReadUAV != ValuesWriteUAV, TEXT("BitonicSort: read/write value UAVs must back different resources. "));

	TArray<FRHITransitionInfo, TInlineAllocator<4>> Transitions;
	Transitions.Add(FRHITransitionInfo(SortBuffers.RemoteKeyUAVs[SrcBuf], ERHIAccess::Unknown, ERHIAccess::SRVCompute));
	Transitions.Add(FRHITransitionInfo(SortBuffers.RemoteKeyUAVs[DstBuf], ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	Transitions.Add(FRHITransitionInfo(ValuesWriteUAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	Transitions.Add(FRHITransitionInfo(ValuesReadUAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	RHICmdList.Transition(Transitions);

	SetComputePipelineState(RHICmdList, ComputeShader);

	SetShaderParametersLegacyCS(RHICmdList, ShaderRef,
		UniformBuffer,
		SortBuffers.RemoteKeySRVs[SrcBuf],
		ValuesReadUAV,
		SortBuffers.RemoteKeyUAVs[DstBuf],
		ValuesWriteUAV);

	DispatchComputeShader(RHICmdList, ShaderRef.GetShader(), GroupCount, 1, 1);

	UnsetShaderParametersLegacyCS(RHICmdList, ShaderRef);

	RHICmdList.Transition({
		FRHITransitionInfo(SortBuffers.RemoteKeyUAVs[DstBuf], ERHIAccess::UAVCompute, ERHIAccess::SRVCompute),
		FRHITransitionInfo(ValuesWriteUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute)
	});
}

int32 BitonicSortGPUBuffers(
	FRHICommandList& RHICmdList,
	FGPUSortBuffers SortBuffers,
	int32 BufferIndex,
	int32 Count,
	ERHIFeatureLevel::Type FeatureLevel)
{
	if (Count <= 1)
	{
		return BufferIndex;
	}

	// Perf waring due O(n log^2 n) complexity.
	ensureMsgf(Count <= (1 << 24), TEXT("BitonicSortGPUBuffers: Count %d exceeds 16M. Radix sort will likely perform better at this scale."), Count);

	SCOPED_DRAW_EVENTF(RHICmdList, BitonicSortGPU, TEXT("BitonicSort(%d)"), Count);

	const uint32 PaddedCount = FMath::RoundUpToPowerOfTwo((uint32)Count);
	const uint32 BlockSize = BITONIC_BLOCK_SIZE;
	const uint32 Log2Block = FMath::FloorLog2(BlockSize);
	const uint32 Log2Padded = FMath::FloorLog2(PaddedCount);

	auto ShaderMap = GetGlobalShaderMap(FeatureLevel);
	const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];
	const bool bUseWaveOps = UseWaveOps(ShaderPlatform);

	FBitonicSortLocalSortCS::FPermutationDomain LocalSortPermutation;
	LocalSortPermutation.Set<FBitonicUseWaveOps>(bUseWaveOps);
	TShaderMapRef<FBitonicSortLocalSortCS> LocalSortCS(ShaderMap, LocalSortPermutation);

	TShaderMapRef<FBitonicSortGlobalMergeCS> GlobalMergeCS(ShaderMap);

	FBitonicSortLocalMergeCS::FPermutationDomain LocalMergePermutation;
	LocalMergePermutation.Set<FBitonicUseWaveOps>(bUseWaveOps);
	TShaderMapRef<FBitonicSortLocalMergeCS> LocalMergeCS(ShaderMap, LocalMergePermutation);

	int32 PassIndex = 0;

	// Total buffer-flipping passes (global merges are in-place): 1 local sort + 1 local merge per stage.
	int32 TotalPasses = 1;
	for (uint32 Stage = Log2Block + 1; Stage <= Log2Padded; ++Stage)
	{
		++TotalPasses;
	}

	const bool bSinglePassNeedsPostCopy = (TotalPasses == 1) && (SortBuffers.FinalValuesUAV != nullptr);

	auto GetValuesReadUAV = [&](int32 Pass, int32 SrcBuf) -> FRHIUnorderedAccessView*
	{
		if (Pass == 0 && SortBuffers.FirstValuesSRV)
		{
			checkf(SortBuffers.FinalValuesUAV, TEXT("BitonicSort: FirstValuesSRV requires FinalValuesUAV."));
			return SortBuffers.FinalValuesUAV;
		}
		return SortBuffers.RemoteValueUAVs[SrcBuf];
	};

	auto GetValuesWriteUAV = [&](int32 Pass, int32 DstBuf) -> FRHIUnorderedAccessView*
	{
		if (Pass == TotalPasses - 1 && SortBuffers.FinalValuesUAV && !bSinglePassNeedsPostCopy)
		{
			return SortBuffers.FinalValuesUAV;
		}
		return SortBuffers.RemoteValueUAVs[DstBuf];
	};

	{
		FBitonicSortParameters Params;
		Params.BitonicSort_Count = (uint32)Count;
		Params.BitonicSort_PaddedCount = PaddedCount;
		Params.BitonicSort_StageBlockSize = 0; // Not used by local sort
		Params.BitonicSort_StepBlockSize = 0;  // Not used by local sort
		FBitonicSortUniformBufferRef UniformBuffer = FBitonicSortUniformBufferRef::CreateUniformBufferImmediate(Params, UniformBuffer_SingleDraw);

		const int32 SrcBuf = BufferIndex;
		const int32 DstBuf = BufferIndex ^ 1;
		// Dispatch enough groups to cover PaddedCount (sentinels must be written).
		const int32 NumGroups = FMath::Max(1, (int32)(PaddedCount / BlockSize));

		FRHIUnorderedAccessView* ValuesReadUAV  = GetValuesReadUAV(PassIndex, SrcBuf);
		FRHIUnorderedAccessView* ValuesWriteUAV = GetValuesWriteUAV(PassIndex, DstBuf);

		DispatchBitonicKernel(
			RHICmdList,
			LocalSortCS.GetComputeShader(),
			LocalSortCS,
			UniformBuffer,
			SortBuffers,
			SrcBuf,
			DstBuf,
			ValuesReadUAV,
			ValuesWriteUAV,
			NumGroups);

		BufferIndex ^= 1;
		++PassIndex;

		// Single pass wrote to scratch
		if (bSinglePassNeedsPostCopy)
		{
			RHICmdList.Transition({
				FRHITransitionInfo(SortBuffers.RemoteValueUAVs[DstBuf], ERHIAccess::UAVCompute, ERHIAccess::SRVCompute),
				FRHITransitionInfo(SortBuffers.FinalValuesUAV,          ERHIAccess::Unknown,     ERHIAccess::UAVCompute)
			});

			FRHIUnorderedAccessView* TargetUAV = SortBuffers.FinalValuesUAV;
			int32 TargetSize = Count;
			CopyUIntBufferToTargets(RHICmdList, FeatureLevel, SortBuffers.RemoteValueSRVs[DstBuf], &TargetUAV, &TargetSize, 0, 1);
		}
	}

	bool bGlobalMergePSOBound = false;
	bool bBuffersInUAVState = false;

	// Global-merge constants
	const int32 NumPairs = (int32)(PaddedCount / 2);
	const int32 PairsPerGroup = BITONIC_THREAD_COUNT * BITONIC_GLOBAL_MERGE_EPT;
	const int32 GlobalMergeGroups = FMath::Max(1, FMath::DivideAndRoundUp(NumPairs, PairsPerGroup));

	for (uint32 Stage = Log2Block + 1; Stage <= Log2Padded; ++Stage)
	{
		const uint32 StageBlockSize = 1u << Stage;

		for (uint32 Step = Stage; Step >= 1; --Step)
		{
			const uint32 StepBlockSize = 1u << Step;

			if (StepBlockSize <= BlockSize)
			{
				// Local merge: handles this step and all remaining smaller steps in shared memory.
				FBitonicSortParameters Params;
				Params.BitonicSort_Count = (uint32)Count;
				Params.BitonicSort_PaddedCount = PaddedCount;
				Params.BitonicSort_StageBlockSize = StageBlockSize;
				Params.BitonicSort_StepBlockSize = StepBlockSize;
				FBitonicSortUniformBufferRef UniformBuffer = FBitonicSortUniformBufferRef::CreateUniformBufferImmediate(Params, UniformBuffer_SingleDraw);

				const int32 SrcBuf = BufferIndex;
				const int32 DstBuf = BufferIndex ^ 1;
				const int32 NumGroups = FMath::Max(1, (int32)(PaddedCount / BlockSize));

				FRHIUnorderedAccessView* ValuesReadUAV  = GetValuesReadUAV(PassIndex, SrcBuf);
				FRHIUnorderedAccessView* ValuesWriteUAV = GetValuesWriteUAV(PassIndex, DstBuf);

				DispatchBitonicKernel(
					RHICmdList,
					LocalMergeCS.GetComputeShader(),
					LocalMergeCS,
					UniformBuffer,
					SortBuffers,
					SrcBuf,
					DstBuf,
					ValuesReadUAV,
					ValuesWriteUAV,
					NumGroups);

				BufferIndex ^= 1;
				++PassIndex;

				// Local merge changed PSO and buffer states
				bGlobalMergePSOBound = false;
				bBuffersInUAVState = false;
				break;
			}
			else
			{
				// Global merge: in-place compare-swap; PSO and transitions hoisted across consecutive global-merge dispatches.
				FBitonicSortParameters Params;
				Params.BitonicSort_Count = (uint32)Count;
				Params.BitonicSort_PaddedCount = PaddedCount;
				Params.BitonicSort_StageBlockSize = StageBlockSize;
				Params.BitonicSort_StepBlockSize = StepBlockSize;
				FBitonicSortUniformBufferRef UniformBuffer = FBitonicSortUniformBufferRef::CreateUniformBufferImmediate(Params, UniformBuffer_SingleDraw);

				if (!bGlobalMergePSOBound)
				{
					SetComputePipelineState(RHICmdList, GlobalMergeCS.GetComputeShader());
					bGlobalMergePSOBound = true;
				}

				if (!bBuffersInUAVState)
				{
					RHICmdList.Transition({
						FRHITransitionInfo(SortBuffers.RemoteKeyUAVs[BufferIndex],   ERHIAccess::SRVCompute, ERHIAccess::UAVCompute),
						FRHITransitionInfo(SortBuffers.RemoteValueUAVs[BufferIndex], ERHIAccess::UAVCompute, ERHIAccess::UAVCompute)
					});
					bBuffersInUAVState = true;
				}
				else
				{
					RHICmdList.Transition({
						FRHITransitionInfo(SortBuffers.RemoteKeyUAVs[BufferIndex], ERHIAccess::UAVCompute, ERHIAccess::UAVCompute),
						FRHITransitionInfo(SortBuffers.RemoteValueUAVs[BufferIndex], ERHIAccess::UAVCompute, ERHIAccess::UAVCompute)
					});
				}

				SetShaderParametersLegacyCS(RHICmdList, GlobalMergeCS,
					UniformBuffer,
					SortBuffers.RemoteKeyUAVs[BufferIndex],
					SortBuffers.RemoteValueUAVs[BufferIndex]);

				DispatchComputeShader(RHICmdList, GlobalMergeCS.GetShader(), GlobalMergeGroups, 1, 1);

				UnsetShaderParametersLegacyCS(RHICmdList, GlobalMergeCS);
			}
		}
	}

	FRHIUnorderedAccessView* const FinalValuesUAVResult = SortBuffers.FinalValuesUAV
		? SortBuffers.FinalValuesUAV
		: SortBuffers.RemoteValueUAVs[BufferIndex];
	RHICmdList.Transition(FRHITransitionInfo(FinalValuesUAVResult, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute));

	return BufferIndex;
}
